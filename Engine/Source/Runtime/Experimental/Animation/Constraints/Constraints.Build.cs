// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class Constraints : ModuleRules
{
	public Constraints(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"AnimationCore",
				"MovieScene",
				"Slate",
				"SlateCore"
			}
		);
	}
}