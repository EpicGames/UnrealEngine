// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConstraintChannel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConstraintChannel)

bool FMovieSceneConstraintChannel::Evaluate(FFrameTime InTime, bool& OutValue) const
{
	if (Times.IsEmpty())
	{
		if (bHasDefaultValue)
		{
			OutValue = DefaultValue;
			return true;
		}
		return false;
	}

	if (InTime.FrameNumber < Times[0])
	{
		if (bHasDefaultValue && !Values.IsEmpty() && !Values[0])
		{
			OutValue = DefaultValue;
			return true;
		}
		return false;
	}

	return FMovieSceneBoolChannel::Evaluate(InTime, OutValue);
}

bool FMovieSceneConstraintChannel::IsInfinite() const
{
	return Times.IsEmpty() && bHasDefaultValue && DefaultValue;
}