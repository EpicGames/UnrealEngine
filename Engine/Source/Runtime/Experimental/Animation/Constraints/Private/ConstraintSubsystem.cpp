// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConstraintSubsystem.h"
#include "Containers/Ticker.h"
#include "Engine/Engine.h"
#include "ConstraintsManager.h"
#include "Algo/ForEach.h"
#include "Misc/CoreDelegates.h"
#include "Transform/TransformConstraint.h"

//needs to be static to avoid system getting deleted with dangling handles.

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConstraintSubsystem)
FDelegateHandle UConstraintSubsystem::OnWorldInitHandle;
FDelegateHandle UConstraintSubsystem::OnWorldCleanupHandle;
FDelegateHandle UConstraintSubsystem::OnPostGarbageCollectHandle;

UConstraintSubsystem::UConstraintSubsystem()
{
}

void UConstraintSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	if (GEngine && GEngine->IsInitialized())
	{
		InitializeInternal();
	}
	else
	{
		FCoreDelegates::GetOnPostEngineInit().AddUObject(this, &UConstraintSubsystem::InitializeInternal);
	}
	
	SetFlags(RF_Transactional);
}

void UConstraintSubsystem::InitializeInternal()
{
	RegisterConfig<UTickableTranslationConstraint>();
	RegisterConfig<UTickableRotationConstraint>();
	RegisterConfig<UTickableScaleConstraint>();
	RegisterConfig<UTickableParentConstraint>();
	RegisterConfig<UTickableLookAtConstraint>();
	
	OnWorldInitHandle = FWorldDelegates::OnPreWorldInitialization.AddStatic(&UConstraintSubsystem::OnWorldInit);
	OnWorldCleanupHandle = FWorldDelegates::OnWorldCleanup.AddStatic(&UConstraintSubsystem::OnWorldCleanup);
	OnPostGarbageCollectHandle = FCoreUObjectDelegates::GetPostGarbageCollect().AddStatic(&UConstraintSubsystem::OnPostGarbageCollect);
	
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
}

UTickableConstraint* UConstraintSubsystem::GetConfig(const UClass* InConstraintClass) 
{
	const TObjectPtr<UTickableConstraint>* ObjectKey = ConstraintsConfig.Find(InConstraintClass);
	ensure(ObjectKey && ObjectKey->Get());
	return ObjectKey ? ObjectKey->Get() : nullptr;
}

void UConstraintSubsystem::Deinitialize()
{
	for (int32 Index = ConstraintsInWorld.Num() - 1; Index >= 0; --Index)
	{
		ConstraintsInWorld[Index].RemoveConstraints(ConstraintsInWorld[Index].World.Get());
	}
	ConstraintsInWorld.Reset();

	FWorldDelegates::OnPreWorldInitialization.Remove(OnWorldInitHandle);
	FWorldDelegates::OnWorldCleanup.Remove(OnWorldCleanupHandle);
	FCoreUObjectDelegates::GetPostGarbageCollect().Remove(OnPostGarbageCollectHandle);

	Super::Deinitialize();
}

UConstraintSubsystem* UConstraintSubsystem::Get()
{
	if (GEngine && GEngine->IsInitialized())
	{
		return GEngine->GetEngineSubsystem<UConstraintSubsystem>();
	}
	return nullptr;
}

int32 UConstraintSubsystem::GetConstraintsInWorldIndex(const UWorld* InWorld) const
{
	return ConstraintsInWorld.IndexOfByPredicate([InWorld](const FConstraintsInWorld& ConstInWorld)
	{
		return ConstInWorld.World.Get() == InWorld; 
	});
}

const FConstraintsInWorld* UConstraintSubsystem::ConstraintsInWorldFind(const UWorld* InWorld) const
{
	if (bNeedsCleanup)
	{
		CleanupInvalidConstraints();
	}

	const int32 Index = GetConstraintsInWorldIndex(InWorld);
	return Index != INDEX_NONE ? &ConstraintsInWorld[Index] : nullptr;
}

FConstraintsInWorld* UConstraintSubsystem::ConstraintsInWorldFind(const UWorld* InWorld) 
{
	if (bNeedsCleanup)
	{
		CleanupInvalidConstraints();
	}

	const int32 Index = GetConstraintsInWorldIndex(InWorld);
	return Index != INDEX_NONE ? &ConstraintsInWorld[Index] : nullptr;
}

FConstraintsInWorld& UConstraintSubsystem::ConstraintsInWorldFindOrAdd(UWorld* InWorld)
{
	if (bNeedsCleanup)
	{
		CleanupInvalidConstraints();
	}

	const int32 Index = GetConstraintsInWorldIndex(InWorld);
	if (Index != INDEX_NONE)
	{
		return ConstraintsInWorld[Index];
	}
	
	FConstraintsInWorld& NewConstraintsInWorld = ConstraintsInWorld.Emplace_GetRef();
	NewConstraintsInWorld.World = InWorld;

	Algo::ForEach(ConstraintsInWorld, [](FConstraintsInWorld& ConstInWorld)
	{
		return ConstInWorld.InvalidateGraph();
	});
	
	return NewConstraintsInWorld;
}

TArray<TWeakObjectPtr<UTickableConstraint>> UConstraintSubsystem::GetConstraints(const UWorld* InWorld) const
{
	static const TArray< TWeakObjectPtr<UTickableConstraint> > DummyArray;
	if (const FConstraintsInWorld* Constraints = ConstraintsInWorldFind(InWorld))
	{
		return (Constraints->Constraints);
	}
	return DummyArray;
}

const TArray<TWeakObjectPtr<UTickableConstraint>>& UConstraintSubsystem::GetConstraintsArray(const UWorld* InWorld) const
{
	static const TArray< TWeakObjectPtr<UTickableConstraint> > DummyArray;
	if (const FConstraintsInWorld* Constraints = ConstraintsInWorldFind(InWorld))
	{
		return (Constraints->Constraints);
	}
	return DummyArray;
}

void UConstraintSubsystem::AddConstraint(UWorld* InWorld, UTickableConstraint* InConstraint)
{	
	Modify();
	FConstraintsInWorld& Constraints = ConstraintsInWorldFindOrAdd(InWorld);
	if (Constraints.Constraints.Contains(InConstraint) == false)
	{
		Constraints.Constraints.Emplace(InConstraint);
		Constraints.InvalidateGraph();
	}
	OnConstraintAddedToSystem_BP.Broadcast(this, InConstraint);
}

void UConstraintSubsystem::RemoveConstraint(UWorld* InWorld, UTickableConstraint* InConstraint, bool bDoNoCompensate)
{
	Modify();
	OnConstraintRemovedFromSystem_BP.Broadcast(this, InConstraint, bDoNoCompensate);

	// store tick function so that it can be removed from prerequisites
	const FConstraintTickFunction* RemovedTickFunction = InConstraint->FindTickFunction(InWorld);
	
	// disable constraint
	InConstraint->Modify();
	InConstraint->TeardownConstraint(InWorld);

	if (FConstraintsInWorld* Constraints = ConstraintsInWorldFind(InWorld))
	{
		Constraints->Constraints.Remove(InConstraint);		
		Constraints->InvalidateGraph();

		// remove useless tick prerequisites on remaining constraints
		const bool bTickFunctionRemoved = RemovedTickFunction && !InConstraint->FindTickFunction(InWorld);
		if (bTickFunctionRemoved)
		{
			static constexpr bool bEvenIfPendingKill = true;
			for (const TWeakObjectPtr<UTickableConstraint>& OtherConstraint: Constraints->Constraints)
			{
				if (OtherConstraint.IsValid() && !OtherConstraint.IsStale())
				{
					if (FConstraintTickFunction* OtherTickFunction = OtherConstraint->FindTickFunction(InWorld))
					{
						TArray<FTickPrerequisite>& Prerequisites = OtherTickFunction->GetPrerequisites();
						for (int32 PrereqIndex = 0; PrereqIndex < Prerequisites.Num(); PrereqIndex++)
						{
							FTickPrerequisite& Prerequisite = Prerequisites[PrereqIndex];
							UObject* PrereqObject = Prerequisite.PrerequisiteObject.Get(bEvenIfPendingKill);
							if (!PrereqObject)
							{
								// remove prerequisite coming from a stale object (cf. FTickFunction::QueueTickFunction)
								Prerequisites.RemoveAtSwap(PrereqIndex--);
							}
							else if (PrereqObject == this && Prerequisite.Get() == RemovedTickFunction)
							{
								// remove prerequisite from the constraint just removed (cf. UConstraintSubsystem::SetConstraintDependencies) 
								Prerequisites.RemoveAtSwap(PrereqIndex--);
							}
						}	
					}
				}
			}
		}
	}
}

// we want InFunctionToTickBefore to tick first = InFunctionToTickBefore is a prerex of InFunctionToTickAfter
void UConstraintSubsystem::SetConstraintDependencies(
	FConstraintTickFunction* InFunctionToTickBefore,
	FConstraintTickFunction* InFunctionToTickAfter)
{
	if (!InFunctionToTickBefore || !InFunctionToTickAfter || InFunctionToTickBefore == InFunctionToTickAfter)
	{
		return;
	}
	
	// look for child tick function in in parent's prerequisites. 
	const TArray<FTickPrerequisite>& ParentPrerequisites = InFunctionToTickAfter->GetPrerequisites();
	const bool bIsChildAPrerexOfParent = ParentPrerequisites.ContainsByPredicate([InFunctionToTickBefore](const FTickPrerequisite& Prerex)
		{
			return Prerex.PrerequisiteTickFunction == InFunctionToTickBefore;
		});

	// child tick function is already a prerex -> parent already ticks after child
	if (bIsChildAPrerexOfParent)
	{
		return;
	}

	// look for parent tick function in in child's prerequisites
	const TArray<FTickPrerequisite>& ChildPrerequisites = InFunctionToTickBefore->GetPrerequisites();
	const bool bIsParentAPrerexOfChild = ChildPrerequisites.ContainsByPredicate([InFunctionToTickAfter](const FTickPrerequisite& Prerex)
		{
			return Prerex.PrerequisiteTickFunction == InFunctionToTickAfter;
		});

	// parent tick function is a prerex of the child tick function (child ticks after parent)
	// so remove it before setting new dependencies.
	if (bIsParentAPrerexOfChild)
	{
		InFunctionToTickBefore->RemovePrerequisite(this, *InFunctionToTickAfter);
	}

	// set dependency
	InFunctionToTickAfter->AddPrerequisite(this, *InFunctionToTickBefore);
}

bool UConstraintSubsystem::HasConstraint(UWorld* InWorld, UTickableConstraint* InConstraint) const
{
	const TArray<TWeakObjectPtr<UTickableConstraint>>& Constraints = GetConstraintsArray(InWorld);
	return  Constraints.Contains(InConstraint);
}

FConstraintsEvaluationGraph& UConstraintSubsystem::GetEvaluationGraph(UWorld* InWorld)
{
	return ConstraintsInWorldFindOrAdd(InWorld).GetEvaluationGraph();
}

void UConstraintSubsystem::InvalidateConstraints()
{
	bNeedsCleanup = true;
}

#if WITH_EDITOR

void UConstraintSubsystem::PostEditUndo()
{
	Super::PostEditUndo();

	for (FConstraintsInWorld& ConstsInWorld: ConstraintsInWorld)
	{
		ConstsInWorld.InvalidateGraph();
	}
	
	FConstraintsManagerController::Notify(EConstraintsManagerNotifyType::ManagerUpdated, this);
}
#endif

void UConstraintSubsystem::OnWorldInit(UWorld* InWorld, const UWorld::InitializationValues)
{
	if (UConstraintSubsystem* System = Get())
	{
		System->ConstraintsInWorldFindOrAdd(InWorld);
	}
}

void UConstraintSubsystem::OnWorldCleanup(UWorld* InWorld, bool bSessionEnded, bool bCleanupResources)
{
	if (UConstraintSubsystem* System = Get())
	{
		const int32 Index = System->GetConstraintsInWorldIndex(InWorld);
		if (Index != INDEX_NONE)
		{
			System->ConstraintsInWorld[Index].RemoveConstraints(InWorld);
			System->ConstraintsInWorld.RemoveAt(Index);

			Algo::ForEach(System->ConstraintsInWorld, [](FConstraintsInWorld& ConstInWorld)
			{
				return ConstInWorld.InvalidateGraph();
			});
		}
	}
}

void UConstraintSubsystem::OnPostGarbageCollect()
{
	if (UConstraintSubsystem* System = Get())
	{
		System->InvalidateConstraints();
	}
}

void UConstraintSubsystem::CleanupInvalidConstraints() const
{
	if (UConstraintSubsystem* System = Get())
	{
		for (FConstraintsInWorld& WorldConstraints: System->ConstraintsInWorld)
		{
			UWorld* World = WorldConstraints.World.Get();
			
			TSet<FTickFunction*> ConstraintsTickFunctions;
			ConstraintsTickFunctions.Reserve(WorldConstraints.Constraints.Num());
			
			// remove stale constraints and store valid constraints tick functions
			WorldConstraints.Constraints.RemoveAll([World, &ConstraintsTickFunctions](const TWeakObjectPtr<UTickableConstraint>& InConstraint)
			{
				const bool bRemove = !InConstraint.IsValid() || InConstraint.IsStale();
				if (!bRemove && World)
				{
					// store tick functions
					ConstraintsTickFunctions.Add(&InConstraint->GetTickFunction(World));
				}
				return bRemove;
			});

			if (World)
			{
				static constexpr bool bEvenIfPendingKill = true;
				
				// cleanup useless tick prerequisites
				for (const TWeakObjectPtr<UTickableConstraint>& Constraint : WorldConstraints.Constraints)
				{
					TArray<FTickPrerequisite>& Prerequisites = Constraint->GetTickFunction(World).GetPrerequisites();
					for (int32 PrereqIndex = 0; PrereqIndex < Prerequisites.Num(); PrereqIndex++)
					{
						FTickPrerequisite& Prerequisite = Prerequisites[PrereqIndex];
						
						UObject* PrereqObject = Prerequisite.PrerequisiteObject.Get(bEvenIfPendingKill);
						if (!PrereqObject)
						{
							// remove prerequisite coming from stale object (cf. FTickFunction::QueueTickFunction)
							Prerequisites.RemoveAtSwap(PrereqIndex--);
						}
						else if (PrereqObject == System && !ConstraintsTickFunctions.Contains(Prerequisite.PrerequisiteTickFunction))
						{
							// remove prerequisite coming from GCd constraint (cf. UConstraintSubsystem::SetConstraintDependencies) 
							Prerequisites.RemoveAtSwap(PrereqIndex--);
						}
					}
				}
			}
			
			WorldConstraints.InvalidateGraph();
		}
		bNeedsCleanup = false;
	}
}

/*************************************
*FConstraintsInWorld
*************************************/

void FConstraintsInWorld::RemoveConstraints(UWorld* InWorld)
{
	for (TWeakObjectPtr<UTickableConstraint>& Constraint : Constraints)
	{
		if (Constraint.IsValid())
		{
			Constraint->TeardownConstraint(InWorld);
		}
	}
	Constraints.SetNum(0);
	InvalidateGraph();
}

FConstraintsEvaluationGraph& FConstraintsInWorld::GetEvaluationGraph()
{
	if (!EvaluationGraph)
	{
		EvaluationGraph = MakeShared<FConstraintsEvaluationGraph>(this);
	}
	return *EvaluationGraph;
}

void FConstraintsInWorld::InvalidateGraph()
{
	if (EvaluationGraph)
	{
		EvaluationGraph.Reset();
	}
	FConstraintsManagerController::Notify(EConstraintsManagerNotifyType::GraphUpdated, nullptr);
}
