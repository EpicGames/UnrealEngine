// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ConstraintsEvaluationGraph.h"
#include "UObject/Object.h"
#include "Engine/EngineBaseTypes.h"
#include "Subsystems/EngineSubsystem.h"
#include "ConstraintsManager.h"
#include "Engine/World.h"
#include "ConstraintSubsystem.generated.h"

class UTickableConstraint;
class ULevel;

USTRUCT()
struct  FConstraintsInWorld 
{
public:
	GENERATED_BODY()

	UPROPERTY(transient)
	TWeakObjectPtr<UWorld> World;

	UPROPERTY(transient)
	mutable TArray<TWeakObjectPtr<UTickableConstraint>> Constraints;

	CONSTRAINTS_API void RemoveConstraints(UWorld* World);

	FConstraintsEvaluationGraph& GetEvaluationGraph();
	void InvalidateGraph();

private:
	TSharedPtr<FConstraintsEvaluationGraph> EvaluationGraph = nullptr;
};


UCLASS()
class  UConstraintSubsystem: public UEngineSubsystem
{
	GENERATED_BODY()

public:	
	/** Dynamic blueprintable delegates for knowing when a constraints are added or deleted*/
	DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_TwoParams(FOnConstraintAddedToSystem, UConstraintSubsystem, OnConstraintAddedToSystem_BP, UConstraintSubsystem*, Mananger, UTickableConstraint*, Constraint);
	DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_ThreeParams(FOnConstraintRemovedFromSystem, UConstraintSubsystem, OnConstraintRemovedFromSystem_BP, UConstraintSubsystem*, Mananger, UTickableConstraint*, Constraint, bool, bDoNotCompensate);

	UConstraintSubsystem();

	static UConstraintSubsystem* Get();
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;

	/* Set tick dependencies between two constraints. */
	CONSTRAINTS_API void SetConstraintDependencies(
		FConstraintTickFunction* InFunctionToTickBefore,
		FConstraintTickFunction* InFunctionToTickAfter);

	CONSTRAINTS_API TArray<TWeakObjectPtr<UTickableConstraint>> GetConstraints(const UWorld* InWorld) const;
	CONSTRAINTS_API const TArray<TWeakObjectPtr<UTickableConstraint>>& GetConstraintsArray(const UWorld* InWorld) const;

	CONSTRAINTS_API void AddConstraint(UWorld* InWorld, UTickableConstraint* InConstraint);
	CONSTRAINTS_API void RemoveConstraint(UWorld* InWorld, UTickableConstraint* InConstraint, bool bDoNoCompensate);
	CONSTRAINTS_API bool HasConstraint(UWorld* InWorld, UTickableConstraint* InConstraint) const;

	FConstraintsEvaluationGraph& GetEvaluationGraph(UWorld* InWorld);
	
	void InvalidateConstraints();

#if WITH_EDITOR
	virtual void PostEditUndo() override;
#endif
public:
	/** BP Delegate fired when constraints are added*/
	UPROPERTY(BlueprintAssignable, Category = Constraints, meta = (DisplayName = "OnConstraintAdded"))
	FOnConstraintAddedToSystem OnConstraintAddedToSystem_BP;
	
	/** BP Delegate fired when constraints are removed*/
	UPROPERTY(BlueprintAssignable, Category = Constraints, meta = (DisplayName = "OnConstraintRemoved"))
	FOnConstraintRemovedFromSystem OnConstraintRemovedFromSystem_BP;

	/** Returns a UTickableConstraint instance that can be used as a configuration constraints. */
	UTickableConstraint* GetConfig(const UClass* InConstraintClass);

	/** Register a new constraint class type and allocate a new configuration constraints for that class. */
	template<typename ConstraintType>
	void RegisterConfig()
	{
		static_assert(TIsDerivedFrom<ConstraintType, UTickableConstraint>::Value,
		"The template class ConstraintType must be a subclass of UTickableConstraint.");

		if (UConstraintSubsystem* SubSystem = Get())
		{
			UClass* ConstraintClass = ConstraintType::StaticClass();
			if (ensureAlways(!ConstraintClass->HasAnyClassFlags(CLASS_Abstract)))
			{
				if (!ConstraintsConfig.Contains(ConstraintClass))
				{
					ConstraintsConfig.Emplace(
					ConstraintClass,
					NewObject<ConstraintType>(SubSystem, NAME_None, RF_Transactional));
				}
			}
		}
	}

private:
	UPROPERTY(transient)
	TArray<FConstraintsInWorld> ConstraintsInWorld;
	
	//handles for handling world creation init and teardown
	static FDelegateHandle OnWorldInitHandle;
	static void OnWorldInit(UWorld* World, const UWorld::InitializationValues IVS);
	
	static FDelegateHandle OnWorldCleanupHandle;
	static void OnWorldCleanup(UWorld* World, bool bSessionEnded, bool bCleanupResources);

	//handle for handling GC
	static FDelegateHandle OnPostGarbageCollectHandle;
	static void OnPostGarbageCollect();
	
	void InitializeInternal();

	mutable bool bNeedsCleanup = false;

	const FConstraintsInWorld* ConstraintsInWorldFind(const UWorld* InWorld) const;
	FConstraintsInWorld* ConstraintsInWorldFind(const UWorld* InWorld);
	FConstraintsInWorld& ConstraintsInWorldFindOrAdd(UWorld* InWorld);

	int32 GetConstraintsInWorldIndex(const UWorld* InWorld) const;

	void CleanupInvalidConstraints() const;

	/** Per UClass constraint configuration instance. */
	UPROPERTY(transient)
	TMap<TObjectPtr<UClass>, TObjectPtr<UTickableConstraint>> ConstraintsConfig;
};
