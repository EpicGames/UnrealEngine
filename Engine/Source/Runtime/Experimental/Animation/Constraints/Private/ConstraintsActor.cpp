// Copyright Epic Games, Inc. All Rights Reserved.


#include "ConstraintsActor.h"
#include "ConstraintsManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConstraintsActor)

// Sets default values
AConstraintsActor::AConstraintsActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Set this actor to call Tick() every frame.  You can turn this off to improve performance if you don't need it.
	PrimaryActorTick.bCanEverTick = true;
	
#if WITH_EDITORONLY_DATA
	bLockLocation = true;
	bHiddenEd = true;
#endif // WITH_EDITORONLY_DATA
}

AConstraintsActor::~AConstraintsActor()
{}

void AConstraintsActor::BeginDestroy()
{
	if (IsValid(ConstraintsManager))
	{
		ConstraintsManager->Clear(GetWorld());
	}

	Super::BeginDestroy();
}

void AConstraintsActor::Destroyed()
{
	if (IsValid(ConstraintsManager))
	{
		ConstraintsManager->Clear(GetWorld());
	}
	Super::Destroyed();
}

void AConstraintsActor::BeginPlay()
{
	Super::BeginPlay();
}

void AConstraintsActor::Tick(float DeltaTime)
{	
	Super::Tick(DeltaTime);
}

void AConstraintsActor::PostRegisterAllComponents()
{
	Super::PostRegisterAllComponents();

	if (ConstraintsManager == nullptr)
	{
		ConstraintsManager = NewObject<UConstraintsManager>(this, NAME_None, RF_Transactional);
	}
	ConstraintsManager->Init(GetWorld());
	RegisterConstraintsTickFunctions();
}

void AConstraintsActor::RegisterConstraintsTickFunctions() const
{
	// note ensure that this is not done when useless ( use Level->bIsAssociatingLevel ?)
	if (ULevel* Level = GetLevel())
	{
		if (ConstraintsManager)
		{
			// remove invalid constraints (meaning that both handle's components are invalid, unbound controls are valid)
			static constexpr bool bDeepValidityCheck = false;
			ConstraintsManager->Constraints.RemoveAll([](const TObjectPtr<UTickableConstraint>& InConstraint)
			{
				return !IsValid(InConstraint) || !InConstraint->IsValid(bDeepValidityCheck);
			});
			
			// ensure registration, we now do this by adding it to the controller (which registers and sets it with the subsystem)
			for (const TObjectPtr<UTickableConstraint>& ConstPtr: ConstraintsManager->Constraints)
			{
				FConstraintsManagerController::Get(GetWorld()).AddConstraint(ConstPtr);
			}
		}
	}
}

#if WITH_EDITOR
void AConstraintsActor::PostEditUndo()
{
	Super::PostEditUndo();
}
#endif
