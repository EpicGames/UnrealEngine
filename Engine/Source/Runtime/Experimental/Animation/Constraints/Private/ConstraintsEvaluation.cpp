// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConstraintsEvaluation.h"

#include "ConstraintsManager.h"
#include "Transform/TransformableHandleUtils.h"
#include "Transform/TransformConstraint.h"

namespace UE::Anim::Constraints
{

static bool	bUseFastEvaluation = true;
static FAutoConsoleVariableRef CVarUseFastEvaluation(
	TEXT("Constraints.UseFastEvaluation"),
	bUseFastEvaluation,
	TEXT("Whether to enable fast sequential constraints evaluation. (default is true)") );

bool UseFastEvaluation()
{
	return bUseFastEvaluation && TransformableHandleUtils::SkipTicking();
}

void EvaluateConstraints(TConstArrayView<TWeakObjectPtr<UTickableConstraint>> InConstraints)
{
	if (UseFastEvaluation())
	{
		FSequentialConstraintsEvaluator Evaluator;
		for (const TWeakObjectPtr<UTickableConstraint>& Constraint : InConstraints)
		{
			if (Constraint.IsValid())
			{
				Evaluator.DoEvaluate(Constraint.Get());
			}
		}
	}
	else
	{
		static constexpr bool bTickHandles = true;
		for (const TWeakObjectPtr<UTickableConstraint>& InConstraint : InConstraints)
		{
			if (InConstraint.IsValid())
			{
				InConstraint->Evaluate(bTickHandles);
			}
		}
	}
}
	
}

FSequentialConstraintsEvaluator::FSequentialConstraintsEvaluator(USceneComponent* InInitialComponent)
	: LastMarkedForEvaluation(InInitialComponent)
{}

void FSequentialConstraintsEvaluator::DoEvaluate(UTickableConstraint* InConstraint, const FConstraintTickFunction* InConstraintTickFunction)
{
	if (!ensure(UE::Anim::Constraints::UseFastEvaluation()))
	{
		return;
	}
	
	const bool bIsConstraintActive = InConstraint && InConstraint->IsFullyActive();
	if (!bIsConstraintActive)
	{
		return;
	}
	
	if (InConstraintTickFunction)
	{
		if (!InConstraintTickFunction->IsTickFunctionRegistered() || !InConstraintTickFunction->IsTickFunctionEnabled())
		{
			return;
		}
	}
	
	if (UTickableTransformConstraint* TransformConstraint = Cast<UTickableTransformConstraint>(InConstraint))
	{
		bool bAlsoTick = false;
		if (USceneComponent* Component = TransformableHandleUtils::GetTarget<USceneComponent>(TransformConstraint->ParentTRSHandle))
		{
			if (Component != LastMarkedForEvaluation)
			{
				bAlsoTick = true;
				LastMarkedForEvaluation = Component;
			}
		}
		InConstraint->Evaluate(bAlsoTick);
	}
	else
	{
		static constexpr bool bTickHandles = true;
		InConstraint->Evaluate(bTickHandles);	
	}
}

