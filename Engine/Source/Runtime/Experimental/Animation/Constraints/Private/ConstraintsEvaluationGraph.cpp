// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConstraintsEvaluationGraph.h"

#include "ConstraintsManager.h"
#include "ConstraintSubsystem.h"
#include "Algo/TopologicalSort.h"
#include "Transform/TransformableHandleUtils.h"

namespace ConstraintsEvaluationGraph
{
	
static bool	bDebugGraph = false;
static FAutoConsoleVariableRef CVarDebugEvaluationGraph(
	TEXT("Constraints.DebugEvaluationGraph"),
	bDebugGraph,
	TEXT("Print debug info about constraints evaluation graph.") );
	
}

FConstraintNode& FConstraintsEvaluationGraph::GetNode(const TWeakObjectPtr<UTickableConstraint>& InConstraint)
{
	const int32 Found = Nodes.IndexOfByPredicate([InConstraint](const FConstraintNode& Node)
	{
		return InConstraint.IsValid() && Node.ConstraintID == InConstraint->ConstraintID; 
	});

	if (Found != INDEX_NONE)
	{
		return Nodes[Found];
	}

	FConstraintNode Node;
	Node.ConstraintID = InConstraint->ConstraintID;
	Node.ConstraintTick = &InConstraint->GetTickFunction(ConstraintsInWorld.World.Get());
	return Nodes.Emplace_GetRef(MoveTemp(Node)); 
}

FConstraintNode* FConstraintsEvaluationGraph::FindNode(const TWeakObjectPtr<UTickableConstraint>& InConstraint)
{
	if (!InConstraint.IsValid())
	{
		return nullptr;
	}
	
	return Nodes.FindByPredicate([InConstraint](const FConstraintNode& Node)
	{
		return InConstraint.IsValid() && Node.ConstraintID == InConstraint->ConstraintID; 
	});
}

void FConstraintsEvaluationGraph::FlushPendingEvaluations()
{
	if (State == InvalidData || State == Flushing)
	{
		return;
	}
	
	if (Nodes.IsEmpty())
	{
		return;
	}
	
	if (ConstraintsEvaluationGraph::bDebugGraph)
	{
		UE_LOGF(LogTemp, Warning, "Flush Constraints Evaluation Graph");
	}

	State = Flushing;

	TGuardValue<FSequentialConstraintsEvaluator> SequentialEvaluatorGuard(SequentialEvaluator, nullptr);
	for (FConstraintNode& Node: Nodes)
	{
		if (Node.bMarkedForEvaluation)
		{
			Evaluate(&Node);
		}
	}

	const bool bHasNodesToEvaluate = Nodes.ContainsByPredicate([](const FConstraintNode& Node)
	{
		return Node.bMarkedForEvaluation;
	});
	ensure(!bHasNodesToEvaluate);

	State = ReadyForEvaluation;
}

void FConstraintsEvaluationGraph::Rebuild()
{
	Nodes.Empty();
	SequentialEvaluator = FSequentialConstraintsEvaluator(nullptr);
	
	if (!ensure(ConstraintsInWorld.World.Get()))
	{
		return;
	}

	const TArray<TWeakObjectPtr<UTickableConstraint>>& Constraints = ConstraintsInWorld.Constraints;
	if (Constraints.IsEmpty())
	{
		return;
	}
	
	UE::Constraints::Graph::BuildGraph(ConstraintsInWorld.World.Get(), Constraints, Nodes);
	
	State = ReadyForEvaluation;

	Dump();
}

bool FConstraintsEvaluationGraph::GetSortedConstraints(TArray<TWeakObjectPtr<UTickableConstraint>>& OutConstraints)
{
	OutConstraints.Reset();

	if (!ensure(ConstraintsInWorld.World.Get()))
	{
		return false;
	}

	if (State == InvalidData)
	{
		Rebuild();
	}

	if (Nodes.IsEmpty())
	{
		return false;
	}

	using ConstraintPtr = TWeakObjectPtr<UTickableConstraint>;
	const TArray<ConstraintPtr>& Constraints = ConstraintsInWorld.Constraints;
	
	OutConstraints.Reserve(Nodes.Num());
	for (const FConstraintNode& Node: Nodes)
	{
		const ConstraintPtr Constraint = Constraints[Node.ConstraintIndex];
		if (Constraint.IsValid())
		{
			OutConstraints.Add(Constraint);
		}
	}

	ensure(OutConstraints.Num() == Constraints.Num());
	
	return true;
}

bool FConstraintsEvaluationGraph::IsPendingEvaluation() const
{
	return State == PendingEvaluation;
}

void FConstraintsEvaluationGraph::Evaluate(FConstraintNode* InNode)
{
	const TArray<TWeakObjectPtr<UTickableConstraint>>& Constraints = ConstraintsInWorld.Constraints;
	if (!Constraints.IsValidIndex(InNode->ConstraintIndex))
	{
		return;
	}

	const TWeakObjectPtr<UTickableConstraint> Constraint = Constraints[InNode->ConstraintIndex];
	if (!Constraint.IsValid())
	{
		return;
	}

	if (InNode->bEvaluating)
	{
		return;
	}
	
	// evaluate
	TGuardValue<bool> ReentrantGuard(InNode->bEvaluating, true);
	if (UE::Anim::Constraints::UseFastEvaluation())
	{
		SequentialEvaluator.DoEvaluate(Constraint.Get(), InNode->ConstraintTick);
	}
	else 
	{
		if (Constraint->IsFullyActive() && InNode->ConstraintTick->IsTickFunctionRegistered() && InNode->ConstraintTick->IsTickFunctionEnabled())
		{
			Constraint->Evaluate(TransformableHandleUtils::SkipTicking());
		}
	}
	InNode->bMarkedForEvaluation = false;

	// evaluate dependencies
	for (const uint32 ChildIndex: InNode->Children)
	{
		if (ensure(Nodes.IsValidIndex(ChildIndex)))
		{
			Evaluate(&Nodes[ChildIndex]);
		}
	}
}

void FConstraintsEvaluationGraph::InvalidateData()
{
	State = InvalidData;
	Nodes.Empty();
	SequentialEvaluator = FSequentialConstraintsEvaluator(nullptr);
}

void FConstraintsEvaluationGraph::MarkForEvaluation(const TWeakObjectPtr<UTickableConstraint>& InConstraint)
{
	if (State == InvalidData)
	{
		Rebuild();
	}

	if (State == Flushing)
	{
		// do not mark this constraint for evaluation while flushing.
		// this can happen with UControlRig::OnControlModified being called while evaluating additive rigs
		return;
	}
	
	if (FConstraintNode* Node = FindNode(InConstraint))
	{
		auto GetConstraintLabel = [](const TWeakObjectPtr<UTickableConstraint>& InConstraint)
		{
#if WITH_EDITOR
			return InConstraint->GetFullLabel();
#else
			return InConstraint->GetName();
#endif		
		};

		if (ConstraintsEvaluationGraph::bDebugGraph)
		{
			UE_LOGF(LogTemp, Warning, "Mark %ls For Evaluation", *GetConstraintLabel(InConstraint));
		}
		
		Node->bMarkedForEvaluation = true;
		
		if (State == ReadyForEvaluation)
		{
			State = PendingEvaluation;
		}
	}
}

void FConstraintsEvaluationGraph::Dump() const
{
	if (!ConstraintsEvaluationGraph::bDebugGraph)
	{
		return;
	}
	
	auto GetConstraintLabel = [](const TWeakObjectPtr<UTickableConstraint>& InConstraint)
	{
#if WITH_EDITOR
		return InConstraint->GetFullLabel();
#else
		return InConstraint->GetName();
#endif		
	};
	
	const TArray<TWeakObjectPtr<UTickableConstraint>>& Constraints = ConstraintsInWorld.Constraints;
	UE_LOGF(LogTemp, Warning, "Nb Constraints = %d", Constraints.Num());
	for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ConstraintIndex++)
	{
		if (Constraints[ConstraintIndex].IsValid())
		{
			UE_LOGF(LogTemp, Warning, "\tConstraint[%d] = %ls", ConstraintIndex, *GetConstraintLabel(Constraints[ConstraintIndex]));
		}
	}

	UE_LOGF(LogTemp, Warning, "Nb Nodes = %d", Nodes.Num());
	for (int32 NodeIndex = 0; NodeIndex < Nodes.Num(); NodeIndex++)
	{
		const FConstraintNode& Node = Nodes[NodeIndex];
		ensure(Constraints.IsValidIndex(Node.ConstraintIndex));
		ensure(Node.NodeIndex == NodeIndex);
		UE_LOGF(LogTemp, Warning, "\tNode[%d] = %ls [%d]", NodeIndex, *GetConstraintLabel(Constraints[Node.ConstraintIndex]), Node.ConstraintIndex);
		for (const uint32 ChildIndex: Node.Children)
		{
			if (ensure(Nodes.IsValidIndex(ChildIndex)))
			{
				const FConstraintNode& ChildNode = Nodes[ChildIndex];
				UE_LOGF(LogTemp, Warning, "\t\tChild[%d] = %ls [%d]", ChildIndex, *GetConstraintLabel(Constraints[ChildNode.ConstraintIndex]), ChildNode.ConstraintIndex);
			}
		}
	}
}

namespace UE::Constraints::Graph::Private
{

struct FGraphBuildHelper
{
	FGraphBuildHelper(UWorld* InWorld, const TArrayView<const ConstraintPtr>& InConstraints, TArray<FConstraintNode>& OutNodes)
		: World(InWorld)
		, Constraints(InConstraints)
		, Nodes(OutNodes)
	{}

	void Build()
	{
		Nodes.Reset();
	
		if (!World || Constraints.IsEmpty())
		{
			return;
		}

		BuildNodes();	
		BuildNodeDependencies();
		SortNodes();
	}

private:

	UWorld* World = nullptr;
	const TArrayView<const ConstraintPtr>& Constraints;
	TArray<FConstraintNode>& Nodes;

	FConstraintNode& GetNode(const ConstraintPtr& InConstraint) const
	{
		const int32 Found = Nodes.IndexOfByPredicate([InConstraint](const FConstraintNode& Node)
		{
			return InConstraint.IsValid() && Node.ConstraintID == InConstraint->ConstraintID; 
		});

		if (Found != INDEX_NONE)
		{
			return Nodes[Found];
		}

		FConstraintNode Node;
		Node.ConstraintID = InConstraint->ConstraintID;
		Node.ConstraintTick = &InConstraint->GetTickFunction(World);
		return Nodes.Emplace_GetRef(MoveTemp(Node)); 
	}

	// build vertices
	void BuildNodes() const
	{
		for (int32 ConstraintIndex = 0, NumConstraints = Constraints.Num(), NodeIndex = 0; ConstraintIndex < NumConstraints; ConstraintIndex++)
		{
			if (Constraints[ConstraintIndex].IsValid())
			{
				FConstraintNode& Node = GetNode(Constraints[ConstraintIndex]);
				Node.NodeIndex = NodeIndex++;
				Node.ConstraintIndex = ConstraintIndex;
			}
		}
	}

	// build edges
	void BuildNodeDependencies() const
	{
		for (FConstraintNode& Node: Nodes)
		{
			const TArray<FTickPrerequisite>& Prerequisites = Node.ConstraintTick->GetPrerequisites();
			for (const FTickPrerequisite& Prerex: Prerequisites)
			{
				if (const FTickFunction* PrerexFunction = Prerex.Get())
				{
					FConstraintNode* PrerexNode = Nodes.FindByPredicate([PrerexFunction, Node](const FConstraintNode& OtherNode)
					{
						if (OtherNode.NodeIndex != Node.NodeIndex)
						{
							if (OtherNode.ConstraintTick == PrerexFunction)
							{
								return true;
							}
						}
						return false;
					});

					if (PrerexNode)
					{
						if (!PrerexNode->Parents.Contains(Node.NodeIndex))
						{
							Node.Parents.Add(PrerexNode->NodeIndex);
						}
						else
						{
							// we may have create a cycle
							ensure(false);
						}

						if (!Node.Children.Contains(PrerexNode->NodeIndex))
						{
							PrerexNode->Children.Add(Node.NodeIndex);
						}
						else
						{
							// we may have create a cycle
							ensure(false);
						}
					}
				}
			}
		}
	}
	
	// sort using dependencies
	void SortNodes() const
	{
		const int32 NumNodes = Nodes.Num();

		// switch to Indices
		TArray<int32> Indices;
		Indices.Reserve(NumNodes);
		Algo::Transform(Nodes, Indices, [](const FConstraintNode& Node) { return Node.NodeIndex; });

		// store copy
		const TArray IndicesBeforeSort(Indices);
		const TArray NodesBeforeSort(Nodes);
		
		// try topological sort
		auto GetDependencies = [this](const int32 Index)
		{
			return Nodes[Index].Parents.Array();
		};
		const bool bSucceeded = Algo::TopologicalSort(Indices, GetDependencies);

		TMap<int32, int32> OldToNewIndex;
		
		if (ensure(bSucceeded))
		{
			// store a old to new index map to re-index things out
			bool bHasBeenReordered = false;
			for (int32 Index = 0; Index < NumNodes; ++Index)
			{
				const int32 OldIndex = IndicesBeforeSort.IndexOfByKey(Indices[Index]);
				OldToNewIndex.Emplace(OldIndex, Index);
				bHasBeenReordered |= OldIndex != Index;
			}

			// early exits if it hasn't been reordered 
			if (!bHasBeenReordered)
			{
				if (ConstraintsEvaluationGraph::bDebugGraph)
				{
					UE_LOGF(LogTemp, Warning, "No need to re-index constraints.");
				}
				return;
			}
			
			// switch back to Nodes
			for (int32 Index = 0; Index < NumNodes; ++Index)
			{
				const int32 NewIndex = Indices[Index];
				Nodes[Index] = NodesBeforeSort[NewIndex];
			}
		}
		else
		{
			// if it failed (mostly due to cycles) return to the basic Algo::Sort to still trying to sort things out.

			auto Predicate = [](const FConstraintNode& LHS, const FConstraintNode& RHS)
			{
				const FConstraintTickFunction* LHSTickFunction = LHS.ConstraintTick;
				const FConstraintTickFunction* RHSTickFunction = RHS.ConstraintTick;
    
				if (!LHSTickFunction || !RHSTickFunction)
				{
					return LHS.ConstraintIndex < RHS.ConstraintIndex;
				}
        
				const TArray<FTickPrerequisite>& RHSPrerex = RHSTickFunction->GetPrerequisites();
				const bool bIsLHSAPrerexOfRHS = RHSPrerex.ContainsByPredicate([LHSTickFunction](const FTickPrerequisite& Prerex)
				{
					return Prerex.PrerequisiteTickFunction == LHSTickFunction;
				});

				if (bIsLHSAPrerexOfRHS)
				{
					return true;
				}
    
				const TArray<FTickPrerequisite>& LHSPrerex = LHSTickFunction->GetPrerequisites();
				const bool bIsRHSAPrerexOfLHS = LHSPrerex.ContainsByPredicate([RHSTickFunction](const FTickPrerequisite& Prerex)
				{
					return Prerex.PrerequisiteTickFunction == RHSTickFunction;
				});
        	
				if (bIsRHSAPrerexOfLHS)
				{
					return false;
				}
    
				// if not a prerequisite then compare constraints indices
				return LHS.ConstraintIndex < RHS.ConstraintIndex;
			};

			Algo::Sort(Nodes, Predicate);

			// store old to new index map to re-index things out
			bool bHasBeenReordered = false;
			for (int32 Index = 0; Index < NumNodes; ++Index)
			{
				const int32 OldIndex = IndicesBeforeSort.IndexOfByKey(Nodes[Index].NodeIndex);
				OldToNewIndex.Emplace(OldIndex, Index);
				bHasBeenReordered |= OldIndex != Index;
			}

			// early exits if it hasn't been reordered 
			if (!bHasBeenReordered)
			{
				if (ConstraintsEvaluationGraph::bDebugGraph)
				{
					UE_LOGF(LogTemp, Warning, "No need to re-index constraints.");
				}
				return;
			}
		}

		if (ensure(OldToNewIndex.Num() == NumNodes))
		{
			// reindex node + parents + children
			for (int32 Index = 0; Index < NumNodes; ++Index)
			{
				FConstraintNode& Node = Nodes[Index];
				ReIndexNode(Node, OldToNewIndex);
			}
		}
	}

	static void ReIndexNode(FConstraintNode& InOutNode, const TMap<int32, int32>& InOldToNewIndex)
	{
		// update node + NodeIndex
		InOutNode.NodeIndex = InOldToNewIndex[InOutNode.NodeIndex];

		// update parents indices
		if (!InOutNode.Parents.IsEmpty())
		{
			TSet<uint32> NewParents;
			NewParents.Reserve(InOutNode.Parents.Num());
			for (const int32 OldParentIndex: InOutNode.Parents)
			{
				NewParents.Add(InOldToNewIndex[OldParentIndex]);
			}
			InOutNode.Parents = MoveTemp(NewParents);
		}

		// update children indices
		if (!InOutNode.Children.IsEmpty())
		{
			TSet<uint32> NewChildren;
			NewChildren.Reserve(InOutNode.Children.Num());
			for (const int32 OldChildIndex: InOutNode.Children)
			{
				NewChildren.Add(InOldToNewIndex[OldChildIndex]);
			}	
			InOutNode.Children = MoveTemp(NewChildren);
		}
	}
};

}

namespace UE::Constraints::Graph
{

void BuildGraph(UWorld* InWorld, const TArrayView<const ConstraintPtr>& InConstraints, TArray<FConstraintNode>& OutNodes)
{
	OutNodes.Empty();
	
	if (!ensure(InWorld))
	{
		return;
	}

	if (InConstraints.IsEmpty())
	{
		return;
	}
	
	Private::FGraphBuildHelper BuildHelper(InWorld, InConstraints, OutNodes);
	BuildHelper.Build();
}
	
void SortConstraints(UWorld* InWorld, TArray<ConstraintPtr>& InOutConstraints)
{
	// build graph
	TArray<FConstraintNode> Nodes;
	BuildGraph(InWorld, InOutConstraints, Nodes);

	if (!Nodes.IsEmpty())
	{
		// re-order constraints
		const TArray<ConstraintPtr> ConstraintsCopy(InOutConstraints);
		InOutConstraints.Reset(Nodes.Num());
		for (int32 Index = 0, NumNodes = Nodes.Num(); Index < NumNodes; ++Index)
		{
			const int32 ConstraintIndex = Nodes[Index].ConstraintIndex;
			if (ConstraintsCopy.IsValidIndex(ConstraintIndex))
			{
				InOutConstraints.Add(ConstraintsCopy[ConstraintIndex]);
			}
		}
	}
}

}
