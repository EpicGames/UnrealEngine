// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConstraintsManager.inl"
#include "ConstraintsActor.h"
#include "ConstraintSubsystem.h"
#include "Algo/Copy.h"
#include "Algo/StableSort.h"

#include "Engine/World.h"
#include "Engine/Level.h"
#include "Engine/Engine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConstraintsManager)

/** 
 * FConstraintTickFunction
 **/

FConstraintTickFunction::FConstraintTickFunction()
{
	TickGroup = TG_PrePhysics;
	bCanEverTick = true;
	bStartWithTickEnabled = true;
	bHighPriority = true;
}

FConstraintTickFunction::~FConstraintTickFunction()
{}

FConstraintTickFunction::FConstraintTickFunction(const FConstraintTickFunction& In) : FConstraintTickFunction()
{
	if (In.ConstraintFunctions.Num() > 0) //should only be 1
	{
		ConstraintFunctions.Add(In.ConstraintFunctions[0]);
	}
	Constraint = In.Constraint;
}

void FConstraintTickFunction::ExecuteTick(
	float DeltaTime,
	ELevelTick TickType,
	ENamedThreads::Type CurrentThread,
	const FGraphEventRef& MyCompletionGraphEvent)
{
	EvaluateFunctions();
}

void FConstraintTickFunction::RegisterFunction(ConstraintFunction InConstraint)
{
	if (ConstraintFunctions.IsEmpty())
	{
		ConstraintFunctions.Add(InConstraint);
	}
}

void FConstraintTickFunction::EvaluateFunctions() const
{
	for (const ConstraintFunction& Function: ConstraintFunctions)
	{
		Function();
	}	
}

FString FConstraintTickFunction::DiagnosticMessage()
{
	if(!Constraint.IsValid())
	{
		return FString::Printf(TEXT("FConstraintTickFunction::Tick[%p]"), this);	
	}

#if WITH_EDITOR
	return FString::Printf(TEXT("FConstraintTickFunction::Tick[%p] (%s)"), this, *Constraint->GetLabel());
#else
	return FString::Printf(TEXT("FConstraintTickFunction::Tick[%p] (%s)"), this, *Constraint->GetName());
#endif
}

/** 
 * UTickableConstraint
 **/

FConstraintTickFunction& UTickableConstraint::GetTickFunction(const UWorld* InWorld)
{
	if (!ensure(InWorld && InWorld->GetCurrentLevel()))
	{
		static FConstraintTickFunction DummyFunction;
		return DummyFunction;
	}

	if (TUniquePtr<FConstraintTickFunction>* Found = ConstraintTicks.Find(InWorld->GetCurrentLevel()))
	{
		if (!ensure(Found->Get()))
		{
			(*Found) = MakeUnique<FConstraintTickFunction>();
		}
		return *Found->Get();
	}
	
	TUniquePtr<FConstraintTickFunction>& NewFunction = ConstraintTicks.Emplace(InWorld->GetCurrentLevel(), MakeUnique<FConstraintTickFunction>());
	return *NewFunction.Get();
}

const FConstraintTickFunction& UTickableConstraint::GetTickFunction(const UWorld* InWorld) const
{
	if (!ensure(InWorld && InWorld->GetCurrentLevel()))
	{
		static const FConstraintTickFunction DummyFunction;
		return DummyFunction;
	}
	
	if (TUniquePtr<FConstraintTickFunction>* Found = ConstraintTicks.Find(InWorld->GetCurrentLevel()))
	{
		if (!ensure(Found->Get()))
		{
			(*Found) = MakeUnique<FConstraintTickFunction>();
		}
		return *Found->Get();
	}
	
	TUniquePtr<FConstraintTickFunction>& NewFunction = ConstraintTicks.Emplace(InWorld->GetCurrentLevel(), MakeUnique<FConstraintTickFunction>());
	return *NewFunction.Get();
}

FConstraintTickFunction* UTickableConstraint::FindTickFunction(const UWorld* InWorld) const
{
	if (!ensure(InWorld && InWorld->GetCurrentLevel()))
	{
		return nullptr;
	}
	const TUniquePtr<FConstraintTickFunction>* Found = ConstraintTicks.Find(InWorld->GetCurrentLevel());
	return Found && Found->IsValid() ? Found->Get() : nullptr;
}

void UTickableConstraint::SetActive(const bool bIsActive)
{
	Active = bIsActive;

	for (auto& [WeakLevel, UniqueFunction] : ConstraintTicks)
	{
		if (WeakLevel.IsValid() && UniqueFunction.IsValid())
		{
			UniqueFunction->SetTickFunctionEnable(bIsActive);
		}
	}
}

bool UTickableConstraint::IsFullyActive() const
{
	return Active;
}

void UTickableConstraint::Evaluate(bool bTickHandlesAlso) const
{
	for (const auto& [WeakLevel, UniqueFunction] : ConstraintTicks)
	{
		if (WeakLevel.IsValid() && UniqueFunction.IsValid())
		{
			UniqueFunction->EvaluateFunctions();
		}
	}
}

UTickableConstraint* UTickableConstraint::Duplicate(UObject* NewOuter) const
{
	return DuplicateObject<UTickableConstraint>(this, NewOuter, GetFName());
}

void UTickableConstraint::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	if (!bDuplicateForPIE) //doing actually copy then give unique id
	{
		ConstraintID = FGuid::NewGuid();
	}
}

void UTickableConstraint::PostLoad()
{
	Super::PostLoad();
	
	if (!ConstraintID.IsValid())
	{
		ConstraintID = FGuid::NewGuid();
	}
}

void UTickableConstraint::PostInitProperties()
{
	Super::PostInitProperties();
	
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad) && !ConstraintID.IsValid())
	{
		ConstraintID = FGuid::NewGuid();
	}
}

#if WITH_EDITOR

FString UTickableConstraint::GetLabel() const
{
	return UTickableConstraint::StaticClass()->GetName();
}

FString UTickableConstraint::GetFullLabel() const
{
	return GetLabel();
}

FString UTickableConstraint::GetTypeLabel() const
{
	return GetLabel();
}

void UTickableConstraint::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UTickableConstraint, Active))
	{
		for (auto& [WeakLevel, UniqueFunction] : ConstraintTicks)
		{
			if (WeakLevel.IsValid() && UniqueFunction.IsValid())
			{
				UniqueFunction->SetTickFunctionEnable(Active);
				if (Active)
				{
					UniqueFunction->EvaluateFunctions();
				}
			}	
		}
	}
}

void UTickableConstraint::PostEditUndo()
{
	Super::PostEditUndo();

	// make sure we update ticking when undone
	const bool bActiveTick = ::IsValidChecked(this);
	for (auto& [WeakLevel, UniqueFunction] : ConstraintTicks)
	{
		if (WeakLevel.IsValid() && UniqueFunction.IsValid())
		{
			UniqueFunction->SetTickFunctionEnable(bActiveTick);
			UniqueFunction->bCanEverTick = bActiveTick;
		}
	}
}

#endif

/** 
 * UConstraintsManager
 **/

UConstraintsManager::UConstraintsManager()
{}

UConstraintsManager::~UConstraintsManager()
{

}

void UConstraintsManager::PostLoad()
{
	Super::PostLoad();
	for (TObjectPtr<UTickableConstraint>& ConstPtr : Constraints)
	{
		if (ConstPtr)
		{
			ConstPtr->InitConstraint(GetWorld());
		}
	}
}

void UConstraintsManager::OnActorDestroyed(AActor* InActor)
{
	if (FConstraintsManagerController::bDoNotRemoveConstraint)
	{
		return;
	}
	
	USceneComponent* SceneComponent = InActor ? InActor->GetRootComponent() : nullptr;
	UWorld* ActorWorld = InActor ? InActor->GetWorld() : nullptr;
	if (!ActorWorld || !SceneComponent)
	{
		return;
	}

	TArray< int32 > IndicesToRemove;		
	for (int32 Index = 0; Index < Constraints.Num(); ++Index)
	{
		//if the constraint has a bound object(in sequencer) we don't remove the constraint, it could be a spawnable
		if (Constraints[Index] && IsValid(Constraints[Index].Get()) && Constraints[Index]->HasBoundObjects() == false && Constraints[Index]->ReferencesObject(SceneComponent))
		{
			IndicesToRemove.Add(Index);
		}
	}

	if (!IndicesToRemove.IsEmpty())
	{
		constexpr bool bDoNotCompensate = true;
		for (int32 Index = IndicesToRemove.Num() - 1; Index >= 0; --Index)
		{
			const int32 ConstraintIndex = IndicesToRemove[Index]; 
			if (Constraints.IsValidIndex(ConstraintIndex))
			{
				FConstraintsManagerController::Get(ActorWorld).RemoveConstraint(Constraints[ConstraintIndex], bDoNotCompensate);
			}
		}
	}
}

void UConstraintsManager::RegisterDelegates(UWorld* World)
{
	if (!OnActorDestroyedHandle.IsValid())
	{
		FOnActorDestroyed::FDelegate ActorDestroyedDelegate =
				FOnActorDestroyed::FDelegate::CreateUObject(this, &UConstraintsManager::OnActorDestroyed);
		OnActorDestroyedHandle = World->AddOnActorDestroyedHandler(ActorDestroyedDelegate);
	}
}

void UConstraintsManager::UnregisterDelegates(UWorld* World)
{
	if (World && OnActorDestroyedHandle.IsValid())
	{
		World->RemoveOnActorDestroyedHandler(OnActorDestroyedHandle);
	}
	OnActorDestroyedHandle.Reset();
}

void UConstraintsManager::Init(UWorld* World)
{
	if (World)
	{
		UnregisterDelegates(World);
		RegisterDelegates(World);
	}
}

UConstraintsManager* UConstraintsManager::Get(UWorld* InWorld)
{
	if (!IsValid(InWorld))
	{
		return nullptr;
	}
	// look for ConstraintsActor and return its manager
	if (UConstraintsManager* Manager = Find(InWorld))
	{
		return Manager;
	}

	// create a new ConstraintsActor
	AConstraintsActor* ConstraintsActor = InWorld->SpawnActor<AConstraintsActor>();
#if WITH_EDITOR
	ConstraintsActor->SetActorLabel(TEXT("Constraints Manager"));
#endif // WITH_EDITOR
	return ConstraintsActor->ConstraintsManager;
}

UConstraintsManager* UConstraintsManager::Find(const UWorld* InWorld)
{
	if (!IsValid(InWorld))
	{
		return nullptr;
	}
	
	// should we work with the persistent level?
	const ULevel* Level = InWorld->GetCurrentLevel();

	// look for first ConstraintsActor
	auto FindFirstConstraintsActor = [](const ULevel* Level)
	{
		const int32 Index = Level->Actors.IndexOfByPredicate([](const AActor* Actor)
		{
			return IsValid(Actor) && Actor->IsA<AConstraintsActor>();
		} );

		return Index != INDEX_NONE ? Cast<AConstraintsActor>(Level->Actors[Index]) : nullptr;
	};

	const AConstraintsActor* ConstraintsActor = FindFirstConstraintsActor(Level);
	return ConstraintsActor ? ConstraintsActor->ConstraintsManager : nullptr;
}
void UConstraintsManager::Clear(UWorld* InWorld)
{
	UnregisterDelegates(InWorld);
}
void UConstraintsManager::Dump() const
{
	UE_LOGF(LogTemp, Error, "nb consts = %d", Constraints.Num());
	for (const TObjectPtr<UTickableConstraint>& Constraint: Constraints)
	{
		if (IsValid(Constraint))
		{
			UE_LOGF(LogTemp, Warning, "\t%ls (target hash = %u)",
				*Constraint->GetName(), Constraint->GetTargetHash());
		}
	}
}


/**
 * FConstraintsManagerController
 **/

namespace UE::Private
{

struct FScopedControllerContext
{
	FScopedControllerContext(UWorld* InWorld)
		: World(InWorld)
	{}

	UWorld* GetWorld() const
	{
		return World.IsValid() ? World.Get() : nullptr;
	}

	~FScopedControllerContext()
	{
		if (UWorld* ContextWorld = GetWorld())
		{
			(void)FConstraintsManagerController::Get(ContextWorld);
		}
	}

private:
	TWeakObjectPtr<UWorld> World = nullptr;
};

}

bool FConstraintsManagerController::bDoNotRemoveConstraint = false;

FConstraintsManagerController& FConstraintsManagerController::Get(UWorld* InWorld)
{
	static FConstraintsManagerController Singleton;
	Singleton.World = InWorld;
	return Singleton;
}


UConstraintsManager* FConstraintsManagerController::GetManager() 
{
	if (!World)
	{
		return nullptr;
	}

	const UE::Private::FScopedControllerContext Context(World);
	if (UConstraintsManager* Manager = FindManager())
	{
		return Manager;
	}

	// create a new ConstraintsActor
	AConstraintsActor* ConstraintsActor = Context.GetWorld()->SpawnActor<AConstraintsActor>();
#if WITH_EDITOR
	ConstraintsActor->SetActorLabel(TEXT("Constraints Manager"));
#endif // WITH_EDITOR
	return ConstraintsActor->ConstraintsManager;
}

UConstraintsManager* FConstraintsManagerController::FindManager() const
{
	if (!World)
	{
		return nullptr;
	}
	
	// look for first ConstraintsActor
	auto FindFirstConstraintsActor = [](const ULevel* Level)
	{
		const int32 Index = Level->Actors.IndexOfByPredicate([](const AActor* Actor)
		{
			return IsValid(Actor) && Actor->IsA<AConstraintsActor>();
		} );

		return Index != INDEX_NONE ? Cast<AConstraintsActor>(Level->Actors[Index]) : nullptr;
	};

	// should we work with the persistent level?
	const ULevel* Level = World->GetCurrentLevel();
	const AConstraintsActor* ConstraintsActor = Level ? FindFirstConstraintsActor(Level) : nullptr;
	return ConstraintsActor ? ConstraintsActor->ConstraintsManager : nullptr;
}

void FConstraintsManagerController::DestroyManager() const
{
	if (!World || !World->GetCurrentLevel())
	{
		return;
	}
	
	// note there should be only one...
	TArray<AActor*> ConstraintsActorsToRemove;
	Algo::CopyIf(World->GetCurrentLevel()->Actors, ConstraintsActorsToRemove, [](const AActor* Actor)
	{
		return IsValid(Actor) && Actor->IsA<AConstraintsActor>();
	} );

	for (AActor* ConstraintsActor: ConstraintsActorsToRemove)
	{
		World->DestroyActor(ConstraintsActor, true);
	}
}

void FConstraintsManagerController::StaticConstraintCreated(UWorld* InWorld, UTickableConstraint* InConstraint)
{
	StaticConstraintCreated(InConstraint);
}

void FConstraintsManagerController::StaticConstraintCreated(UTickableConstraint* InConstraint)
{
	if (UConstraintsManager* Manager = GetManager()) //created If needed
	{
		if (Manager->Constraints.Contains(InConstraint) == false)
		{
			Manager->Modify();
			Manager->Constraints.Add(InConstraint);
			InConstraint->Rename(nullptr, Manager, REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
			Manager->OnConstraintAdded_BP.Broadcast(Manager, InConstraint);
		}
	}	
}

bool FConstraintsManagerController::AddConstraint(UTickableConstraint* InConstraint) const
{
	if (!InConstraint || !World)
	{
		return false;
	}
	UConstraintSubsystem* Subsystem = UConstraintSubsystem::Get();
	if (!Subsystem)
	{
		return false;
	}

	{
		const UE::Private::FScopedControllerContext Context(World);
		if (Subsystem->HasConstraint(Context.GetWorld(), InConstraint))
		{
			return false;
		}
		
		InConstraint->InitConstraint(Context.GetWorld());

		Subsystem->AddConstraint(Context.GetWorld(), InConstraint);

		// build dependencies
		InConstraint->AddedToWorld(Context.GetWorld());
	}

	// notify
	Notify(EConstraintsManagerNotifyType::ConstraintAdded, InConstraint);

	return true;
}

int32 FConstraintsManagerController::GetConstraintIndex(const FName& InConstraintName) const
{
	const UConstraintSubsystem* Subsystem = UConstraintSubsystem::Get();
	if (!Subsystem ||!World)
	{
		return INDEX_NONE;
	}

	const UE::Private::FScopedControllerContext Context(World);
	const TArray<TWeakObjectPtr<UTickableConstraint>>& Constraints = Subsystem->GetConstraintsArray(Context.GetWorld());
	return Constraints.IndexOfByPredicate([InConstraintName](const TWeakObjectPtr<UTickableConstraint>& Constraint)
	{
		return 	(Constraint.IsValid() && Constraint->GetFName() == InConstraintName);
	} );
}

	
bool FConstraintsManagerController::RemoveConstraint(UTickableConstraint* InConstraint, bool bDoNotCompensate) 
{
	if (bDoNotRemoveConstraint)
	{
		return false;
	}
	
	UConstraintSubsystem* Subsystem = UConstraintSubsystem::Get();
	if (!Subsystem || !World)
	{
		return false;
	}

	int32 Index = INDEX_NONE;
	{
		const UE::Private::FScopedControllerContext Context(World);
		const TArray<TWeakObjectPtr<UTickableConstraint>>& Constraints = Subsystem->GetConstraintsArray(Context.GetWorld());
		Index = Constraints.IndexOfByPredicate([InConstraint](const TWeakObjectPtr<UTickableConstraint>& Constraint)
		{
			return (Constraint.IsValid() && Constraint.Get() == InConstraint);
		});
	}

	return Index != INDEX_NONE ? RemoveConstraint(Index, bDoNotCompensate) : false;
}

bool FConstraintsManagerController::UnregisterConstraint(UTickableConstraint* InConstraint)
{
	if (!InConstraint || !World)
	{
		return false;
	}
	
	UConstraintSubsystem* Subsystem = UConstraintSubsystem::Get();
	if (!Subsystem)
	{
		return false;
	}

	const UE::Private::FScopedControllerContext Context(World);
	
	static constexpr bool bDoNotCompensate = true;
	if (Subsystem->HasConstraint(Context.GetWorld(), InConstraint))
	{
		Subsystem->RemoveConstraint(Context.GetWorld(), InConstraint, bDoNotCompensate);
		return true;
	}
	return false;
}

bool FConstraintsManagerController::RemoveConstraint(const int32 InConstraintIndex,bool bDoNotCompensate) 
{
	UConstraintSubsystem* Subsystem = UConstraintSubsystem::Get();
	if (!Subsystem || !World)
	{
		return false;
	}

	UTickableConstraint* Constraint = nullptr;
	{
		const UE::Private::FScopedControllerContext Context(World);
		const TArray<TWeakObjectPtr<UTickableConstraint>>& Constraints = Subsystem->GetConstraintsArray(Context.GetWorld());
		if (Constraints.IsValidIndex(InConstraintIndex))
		{
			Constraint = Constraints[InConstraintIndex].Get();
		}
	}

	if (!Constraint)
	{
		return false;
	}

	// delete from subsystem and notify deletion
	{
		auto GetRemoveNotifyType = [bDoNotCompensate]()
		{
			if(bDoNotCompensate)
			{
				return EConstraintsManagerNotifyType::ConstraintRemoved;	
			}
			return EConstraintsManagerNotifyType::ConstraintRemovedWithCompensation;
		};

		const UE::Private::FScopedControllerContext Context(World);
		Notify(GetRemoveNotifyType(), Constraint);	
		Subsystem->Modify();
		Subsystem->RemoveConstraint(Context.GetWorld(), Constraint, bDoNotCompensate);
	}

	if (UConstraintsManager* Manager = FindManager())
	{
		if (Manager->Constraints.Contains(Constraint))
		{
			// delete from manager and notify deletion
			{
				const UE::Private::FScopedControllerContext Context(World);
				Manager->Modify();
				Manager->Constraints.Remove(Constraint);
				Manager->OnConstraintRemoved_BP.Broadcast(Manager, Constraint, bDoNotCompensate);
			}
			
			// destroy constraints actor if no constraints left
			if (Manager->Constraints.IsEmpty())
			{
				DestroyManager();
			}
		}
	}

	return true;
	
}

UTickableConstraint* FConstraintsManagerController::GetConstraint(const FGuid& InGuid) const
{
	const UConstraintSubsystem* Subsystem = UConstraintSubsystem::Get();
	if (!Subsystem || !World)
	{
		return nullptr;
	}

	const UE::Private::FScopedControllerContext Context(World);
	const TArray<TWeakObjectPtr<UTickableConstraint>>& Constraints = Subsystem->GetConstraintsArray(Context.GetWorld());
	const int32 Index = Constraints.IndexOfByPredicate([InGuid](const TWeakObjectPtr<UTickableConstraint>& Constraint)
	{
		return 	(Constraint.IsValid() && Constraint->ConstraintID == InGuid);
	});

	return Constraints.IsValidIndex(Index) ? Constraints[Index].Get() : nullptr;
}

UTickableConstraint* FConstraintsManagerController::GetConstraint(const int32 InConstraintIndex) const
{
	UConstraintSubsystem* Subsystem = UConstraintSubsystem::Get();
	if (!Subsystem || !World)
	{
		return nullptr;
	}

	const UE::Private::FScopedControllerContext Context(World);
	const TArray<TWeakObjectPtr<UTickableConstraint>>& Constraints = Subsystem->GetConstraintsArray(Context.GetWorld());
	if (!Constraints.IsValidIndex(InConstraintIndex))
	{
		return nullptr;	
	}

	return Constraints[InConstraintIndex].Get();
}

TArray< TWeakObjectPtr<UTickableConstraint> > FConstraintsManagerController::GetParentConstraints(
	const uint32 InTargetHash,
	const bool bSorted) const
{
	static const TArray< TWeakObjectPtr<UTickableConstraint> > DummyArray;
	
	if (InTargetHash == 0)
	{
		return DummyArray;
	}

	UConstraintSubsystem* Subsystem = UConstraintSubsystem::Get();
	if (!Subsystem || !World)
	{
		return DummyArray;
	}

	const UE::Private::FScopedControllerContext Context(World);
	const TArray<TWeakObjectPtr<UTickableConstraint>>& Constraints = Subsystem->GetConstraintsArray(Context.GetWorld());

	using ConstraintPtr = TWeakObjectPtr<UTickableConstraint>;

	TArray< TWeakObjectPtr<UTickableConstraint> > FilteredConstraints =
	Constraints.FilterByPredicate( [InTargetHash](const ConstraintPtr& Constraint)
	{
		return Constraint.IsValid() && IsValid(Constraint.Get()) && Constraint->GetTargetHash() == InTargetHash;
	} );
	
	if (bSorted)
	{
		// LHS ticks before RHS = LHS is a prerex of RHS 
		auto TicksBefore = [&Context](const UTickableConstraint& LHS, const UTickableConstraint& RHS)
		{
			if (UWorld* CtxWorld = Context.GetWorld())
			{
				const TArray<FTickPrerequisite>& RHSPrerex = RHS.GetTickFunction(CtxWorld).GetPrerequisites();
				const FConstraintTickFunction& LHSTickFunction = LHS.GetTickFunction(CtxWorld);
				const bool bIsLHSAPrerexOfRHS = RHSPrerex.ContainsByPredicate([&LHSTickFunction](const FTickPrerequisite& Prerex)
				{
					return Prerex.PrerequisiteTickFunction == &LHSTickFunction;
				});
				return bIsLHSAPrerexOfRHS;
			}
			return false;
		};
		
		Algo::StableSort(FilteredConstraints, [TicksBefore](const ConstraintPtr& LHS, const ConstraintPtr& RHS)
		{
			return TicksBefore(*LHS, *RHS);
		});
	}
	
	return FilteredConstraints;
}

void FConstraintsManagerController::SetConstraintsDependencies(
	const FName& InNameToTickBefore,
	const FName& InNameToTickAfter) const
{
	UConstraintSubsystem* Subsystem = UConstraintSubsystem::Get();
	if (!Subsystem || !World)
	{
		return;
	}

	const UE::Private::FScopedControllerContext Context(World);

	const TArray<TWeakObjectPtr<UTickableConstraint>>& Constraints = Subsystem->GetConstraintsArray(Context.GetWorld());

	auto GetIndex = [&Constraints](const FName& InName)
	{
		return Constraints.IndexOfByPredicate([InName](const TWeakObjectPtr<UTickableConstraint>& Constraint)
		{
			return (Constraint.IsValid() && Constraint->GetFName() == InName);
		} );		
	};
	
	const int32 IndexBefore = GetIndex(InNameToTickBefore);
	const int32 IndexAfter = GetIndex(InNameToTickAfter);
	if (!Constraints.IsValidIndex(IndexBefore) || !Constraints.IsValidIndex(IndexAfter) || IndexAfter == IndexBefore)
	{
		return;
	}

	if (UWorld* CtxWorld = Context.GetWorld())
	{
		FConstraintTickFunction& FunctionToTickBefore = Constraints[IndexBefore]->GetTickFunction(CtxWorld);
		FConstraintTickFunction& FunctionToTickAfter = Constraints[IndexAfter]->GetTickFunction(CtxWorld);

		Subsystem->SetConstraintDependencies(&FunctionToTickBefore, &FunctionToTickAfter);

		InvalidateEvaluationGraph();
	}
}

void FConstraintsManagerController::SetConstraintsDependencies(const FGuid& InGuidToTickBefore, const FGuid& InGuidToTickAfter) const
{
	if (!InGuidToTickBefore.IsValid() || !InGuidToTickAfter.IsValid() || InGuidToTickBefore == InGuidToTickAfter)
	{
		return;
	}

	UConstraintSubsystem* Subsystem = UConstraintSubsystem::Get();
	if (!Subsystem || !World)
	{
		return;
	}

	const UE::Private::FScopedControllerContext Context(World);
	const TArray<TWeakObjectPtr<UTickableConstraint>>& Constraints = Subsystem->GetConstraintsArray(Context.GetWorld());
	const int32 IndexBefore = Constraints.IndexOfByPredicate([InGuidToTickBefore](const TWeakObjectPtr<UTickableConstraint>& Constraint)
	{
		return Constraint.IsValid() && Constraint->ConstraintID == InGuidToTickBefore;
	});
	
	const int32 IndexAfter = Constraints.IndexOfByPredicate([InGuidToTickAfter](const TWeakObjectPtr<UTickableConstraint>& Constraint)
	{
		return Constraint.IsValid() && Constraint->ConstraintID == InGuidToTickAfter;
	});
	
	if (IndexBefore == INDEX_NONE || IndexAfter == INDEX_NONE || IndexAfter == IndexBefore)
	{
		return;
	}

	if (UWorld* CtxWorld = Context.GetWorld())
	{
		FConstraintTickFunction& FunctionToTickBefore = Constraints[IndexBefore]->GetTickFunction(CtxWorld);
		FConstraintTickFunction& FunctionToTickAfter = Constraints[IndexAfter]->GetTickFunction(CtxWorld);

		Subsystem->SetConstraintDependencies(&FunctionToTickBefore, &FunctionToTickAfter);

		InvalidateEvaluationGraph();
	}
}

const TArray< TWeakObjectPtr<UTickableConstraint> >& FConstraintsManagerController::GetConstraintsArray() const
{
	UConstraintSubsystem* Subsystem = UConstraintSubsystem::Get();
	if (!Subsystem || !World)
	{
		static const TArray< TWeakObjectPtr<UTickableConstraint> > Empty;
		return Empty;
	}

	const UE::Private::FScopedControllerContext Context(World);
	return Subsystem->GetConstraintsArray(Context.GetWorld());
}

bool FConstraintsManagerController::RemoveAllConstraints(bool bDoNotCompensate) 
{
	UConstraintSubsystem* Subsystem = UConstraintSubsystem::Get();
	if (Subsystem && World)
	{
		Subsystem->Modify();

		TArray<TWeakObjectPtr<UTickableConstraint>> Constraints;
		{
			const UE::Private::FScopedControllerContext Context(World);
			Constraints = Subsystem->GetConstraints(Context.GetWorld());
		}

		bool bRemoved = false;
		for (TWeakObjectPtr<UTickableConstraint>& Constraint : Constraints)
		{
			bRemoved |= RemoveConstraint(Constraint.Get(), bDoNotCompensate);
		}
		return bRemoved;
	}
	
	return false;
}

TArray< TWeakObjectPtr<UTickableConstraint> > FConstraintsManagerController::GetStaticConstraints(const bool bSorted) const
{
	UConstraintsManager* Manager = FindManager();
	if (!Manager)
	{
		static const TArray< TWeakObjectPtr<UTickableConstraint> > Empty;
		return Empty;
	}
	
	// Remove stale constraints. Stale constraints may be caused by unexpected unloading of contributing objects, such as a level sequence.
	Manager->Constraints.RemoveAll([](const TObjectPtr<UTickableConstraint>& ExistingConstraint) -> bool
	{
		return !IsValid(ExistingConstraint);
	});

	TArray< TWeakObjectPtr<UTickableConstraint> > Constraints(Manager->Constraints);
	if (!bSorted)
	{
		return Constraints;
	}

	UE::Constraints::Graph::SortConstraints(World, Constraints);

	return Constraints;
}

TArray< TWeakObjectPtr<UTickableConstraint> > FConstraintsManagerController::GetAllConstraints(const bool bSorted) const
{
	UConstraintSubsystem* Subsystem = UConstraintSubsystem::Get();
	if (!Subsystem || !World)
	{
		static const TArray< TWeakObjectPtr<UTickableConstraint> > Empty;
		return Empty;
	}

	const UE::Private::FScopedControllerContext Context(World);
	
	// use evaluation graph cached data
	if (bSorted)
	{
		TArray<TWeakObjectPtr<UTickableConstraint>> SortedConstraints;
		if (Subsystem->GetEvaluationGraph(Context.GetWorld()).GetSortedConstraints(SortedConstraints))
		{
		   return SortedConstraints;
		}
	}

	TArray<TWeakObjectPtr<UTickableConstraint>> Constraints = Subsystem->GetConstraints(Context.GetWorld());
	// Remove stale constraints (note that GCd constraints should already have been caught by UConstraintSubsystem)
	Constraints.RemoveAll([](const TWeakObjectPtr<UTickableConstraint>& ExistingConstraint) -> bool
	{
		return ExistingConstraint.IsStale();
	});
	return Constraints;
}

void FConstraintsManagerController::EvaluateAllConstraints() const
{
	using ConstraintPtr = TWeakObjectPtr<UTickableConstraint>;

	static constexpr bool bSorted = true;
	const TArray<ConstraintPtr> Constraints = GetAllConstraints(bSorted);
	if (Constraints.IsEmpty())
	{
		return;
	}

	UE::Anim::Constraints::EvaluateConstraints(Constraints);
}

FConstraintsManagerController::FOnSceneComponentConstrained& FConstraintsManagerController::OnSceneComponentConstrained()
{
	static FOnSceneComponentConstrained SceneComponentConstrained;
	return SceneComponentConstrained;
}

FConstraintsManagerNotifyDelegate& FConstraintsManagerController::GetNotifyDelegate()
{
	static FConstraintsManagerNotifyDelegate NotifyDelegate;
	return NotifyDelegate;
}

void FConstraintsManagerController::Notify(EConstraintsManagerNotifyType InNotifyType, UObject* InObject)
{
	switch (InNotifyType)
	{
		case EConstraintsManagerNotifyType::ConstraintAdded: 
		case EConstraintsManagerNotifyType::ConstraintRemoved:
		case EConstraintsManagerNotifyType::ConstraintRemovedWithCompensation:
			checkSlow(Cast<UTickableConstraint>(InObject) != nullptr);
			break;

		case EConstraintsManagerNotifyType::ManagerUpdated:
			checkSlow(Cast<UConstraintSubsystem>(InObject) != nullptr);
			break;
		
		case EConstraintsManagerNotifyType::GraphUpdated:
			break;
		
		default:
			checkfSlow(false, TEXT("Unchecked EConstraintsManagerNotifyType!"));
			break;
	}

	GetNotifyDelegate().Broadcast(InNotifyType, InObject);
}

void FConstraintsManagerController::MarkConstraintForEvaluation(UTickableConstraint* InConstraint) const
{
	UConstraintSubsystem* SubSystem = UConstraintSubsystem::Get();
	if (!SubSystem || !World)
	{
		return;
	}

	const UE::Private::FScopedControllerContext Context(World);
	SubSystem->GetEvaluationGraph(Context.GetWorld()).MarkForEvaluation(InConstraint);
}

void FConstraintsManagerController::InvalidateEvaluationGraph() const
{
	UConstraintSubsystem* SubSystem = UConstraintSubsystem::Get();
	if (!SubSystem || !World)
	{
		return;
	}

	{
		const UE::Private::FScopedControllerContext Context(World);
		SubSystem->GetEvaluationGraph(Context.GetWorld()).InvalidateData();
	}
	
	Notify(EConstraintsManagerNotifyType::GraphUpdated, nullptr);
}

void FConstraintsManagerController::FlushEvaluationGraph() const
{
	UConstraintSubsystem* SubSystem = UConstraintSubsystem::Get();
	if (!SubSystem || !World)
	{
		return;
	}

	const UE::Private::FScopedControllerContext Context(World);
	FConstraintsEvaluationGraph& EvaluationGraph = SubSystem->GetEvaluationGraph(Context.GetWorld());
	if (EvaluationGraph.IsPendingEvaluation())
	{
		EvaluationGraph.FlushPendingEvaluations();
	}
}

bool FConstraintsManagerController::DoesExistInAnyWorld(UTickableConstraint* InConstraint)
{
	bool bFound = false;

	if (InConstraint)
	{
		if (UConstraintSubsystem * Subsystem = UConstraintSubsystem::Get())
		{
			const UE::Private::FScopedControllerContext CtrlContext(World);
			
			for (const FWorldContext& Context : GEngine->GetWorldContexts())
			{
				if (Context.World())
				{
					if (Subsystem->GetConstraintsArray(Context.World()).Find(InConstraint) != INDEX_NONE)
					{
						bFound = true;
						break;
					}
				}
			}
		}
	}
	return bFound;
}