// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "Modules/ModuleInterface.h"

#include "Transform/TransformableRegistry.h"

DECLARE_LOG_CATEGORY_EXTERN(LogConstraints, Log, All);
DEFINE_LOG_CATEGORY(LogConstraints);

class FConstraintsModule : public IModuleInterface
{
public:

	virtual void StartupModule() override
	{
		FTransformableRegistry::RegisterBaseObjects();
	}

	virtual void ShutdownModule() override
	{
		FTransformableRegistry::UnregisterAllObjects();
	}
};

IMPLEMENT_MODULE(FConstraintsModule, Constraints);


