// Copyright Epic Games, Inc. All Rights Reserved.


#include "ConstraintsScripting.h"
#include "ConstraintsManager.h"
#include "Transform/TransformConstraint.h"
#include "Transform/TransformableHandle.h"
#include "Transform/TransformableRegistry.h"
#include "Transform/TransformConstraintUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConstraintsScripting)

UConstraintsManager* UConstraintsScriptingLibrary::GetManager(UWorld* InWorld)
{
	return nullptr;
}

UTransformableComponentHandle* UConstraintsScriptingLibrary::CreateTransformableComponentHandle(
	UWorld* InWorld, USceneComponent* InSceneComponent, const FName& InSocketName)
{
	UTransformableComponentHandle* Handle = UE::TransformConstraintUtil::CreateHandleForSceneComponent(InSceneComponent, InSocketName);
	return Handle;
}

UTransformableHandle* UConstraintsScriptingLibrary::CreateTransformableHandle(UWorld* InWorld, UObject* InObject, const FName& InAttachmentName)
{
	if (!InObject)
	{
		if (InAttachmentName == NAME_None)
		{
			UE_LOGF(LogTemp, Error, "CreateTransformableHandle: InObject in null.");
		}
		else
		{
			UE_LOGF(LogTemp, Error, "CreateTransformableHandle ('%ls'): InObject in null.", *InAttachmentName.ToString());
		}
		return nullptr;	
	}
		
	// look for customized transform handle
	const FTransformableRegistry& Registry = FTransformableRegistry::Get();
	if (const FTransformableRegistry::CreateHandleFuncT CreateFunction = Registry.GetCreateFunction(InObject->GetClass()))
	{
		return CreateFunction(InObject, InAttachmentName);
	}

	UE_LOGF(LogTemp, Error, "CreateTransformableHandle: Object Class '%ls' not supported.", *InObject->GetClass()->GetName());
	return nullptr;
}


UTickableTransformConstraint* UConstraintsScriptingLibrary::CreateFromType(
	UWorld* InWorld,
	const ETransformConstraintType InType)
{
	UTickableTransformConstraint* Constraint = UE::TransformConstraintUtil::CreateFromType(InWorld, InType);
	return Constraint;
}

bool UConstraintsScriptingLibrary::AddConstraint(UWorld* InWorld, UTransformableHandle* InParentHandle, UTransformableHandle* InChildHandle,
	UTickableTransformConstraint *InConstraint, const bool bMaintainOffset)
{
	if (!InWorld)
	{
		UE_LOGF(LogTemp, Error, "AddConstraint: Need Valid World.");
		return false;
	}
	
	if (!InConstraint)
	{
		UE_LOGF(LogTemp, Error, "AddConstraint: InConstraint is null.");
		return false;
	}
	
	const bool bAdded = UE::TransformConstraintUtil::AddConstraint(InWorld, InParentHandle, InChildHandle,InConstraint, bMaintainOffset);
	if (!bAdded)
	{
		UE_LOGF(LogTemp, Error, "AddConstraint: Constraint not added");
		return false;
	}
	
	FConstraintsManagerController& Controller = FConstraintsManagerController::Get(InWorld);
	Controller.StaticConstraintCreated(InConstraint);
	
	return true;
}

TArray<UTickableConstraint*> UConstraintsScriptingLibrary::GetConstraintsArray(UWorld* InWorld)
{
	TArray<UTickableConstraint*> Constraints;
	const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(InWorld);
	const TArray< TWeakObjectPtr<UTickableConstraint> >& ConstraintsArray = Controller.GetConstraintsArray();
	for (const TWeakObjectPtr<UTickableConstraint>& Constraint : ConstraintsArray)
	{
		Constraints.Add(Constraint.Get());
	}
	return Constraints;
}

bool UConstraintsScriptingLibrary::RemoveConstraint(UWorld* InWorld, int32 InIndex)
{
	FConstraintsManagerController& Controller = FConstraintsManagerController::Get(InWorld);
	return Controller.RemoveConstraint(InIndex);
}

bool UConstraintsScriptingLibrary::RemoveThisConstraint(UWorld* InWorld, UTickableConstraint* InTickableConstraint)
{
	FConstraintsManagerController& Controller = FConstraintsManagerController::Get(InWorld);
	const TArray< TWeakObjectPtr<UTickableConstraint> >& ConstraintsArray = Controller.GetConstraintsArray();
	for (int32 Index = 0; Index < ConstraintsArray.Num(); ++Index)
	{
		if (InTickableConstraint == ConstraintsArray[Index].Get())
		{
			return Controller.RemoveConstraint(Index);

		}
	}
	return false;
}
