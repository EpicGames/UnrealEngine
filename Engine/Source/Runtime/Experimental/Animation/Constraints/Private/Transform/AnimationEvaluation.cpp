// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transform/AnimationEvaluation.h"

#include "Animation/AnimInstance.h"
#include "Components/SceneComponent.h"
#include "CoreGlobals.h"
#include "ConstraintsManager.h"
#include "Engine/SkeletalMesh.h"
#include "GameFramework/Actor.h"
#include "Logging/LogMacros.h"
#include "UObject/FastReferenceCollector.h"

namespace UE::Anim
{

static bool	bLogNewEvaluation = false;
static FAutoConsoleVariableRef CVarLogNewEvaluation(
	TEXT("Constraints.LogNewEvaluation"),
	bLogNewEvaluation,
	TEXT("Log new constraints' evaluation scheme.")
);

	
FString GetComponentName(const USceneComponent* InSceneComponent)
{
#if WITH_EDITOR
	const AActor* Actor = InSceneComponent->GetOwner();
	return Actor ? Actor->GetActorLabel() : InSceneComponent->GetName();
#else
	return InSceneComponent->GetName();
#endif
}

FAnimationEvaluator::FAnimationEvaluator(USkeletalMeshComponent* InSkeletalMeshComponent)
	: SkeletalMeshComponent(InSkeletalMeshComponent)
{
	BindDelegates();
}

static float CachingDelta = 0.f;
static int32 CachingCount = 0;
FTransactionallySafeCriticalSection CachingLock;
TSet<USkeletalMeshComponent*> CachedForDeltaTime;

bool IsCaching()
{
	return CachingCount > 0;
}
	
FEvaluationForCachingScope::FEvaluationForCachingScope(const float InDeltaTime)
{
	if (ensure(CachingCount == 0))
	{
		CachingDelta = InDeltaTime;
		{
			TScopeLock ScopeLock(CachingLock);
			CachedForDeltaTime.Reset();
		}
	}
	++CachingCount;
}

FEvaluationForCachingScope::~FEvaluationForCachingScope()
{
	ensure(CachingCount > 0);
	
	--CachingCount;
	if (CachingCount <= 0)
	{
		CachingDelta = 0.f;
		CachingCount = 0;
		{
			TScopeLock ScopeLock(CachingLock);
			CachedForDeltaTime.Reset();
		}
	}
}
	
FAnimationEvaluator::~FAnimationEvaluator()
{
	UnbindDelegates();
	PostEvaluationTasks.Reset();
}
	
void FAnimationEvaluator::Update(const bool bRefreshBoneTransforms)
{
	Context.Clear();
	
	if (bRefreshBoneTransforms)
	{
		RefreshBoneTransforms();
	}
}

bool FAnimationEvaluator::IsValid(const bool bForTransform) const
{
	if  (!SkeletalMeshComponent.IsValid() || Context.ComponentSpaceTransforms.IsEmpty())
	{
		return false;
	}

	if (bForTransform)
	{
		const TArray<FTransform>& ComponentSpaceTransforms = SkeletalMeshComponent->GetComponentSpaceTransforms();
		return Context.ComponentSpaceTransforms.Num() == ComponentSpaceTransforms.Num();
	}
	
	return Context.SkeletalMesh != nullptr;
}

FTransform FAnimationEvaluator::GetGlobalTransform(const FName InSocketName) const
{
	USkeletalMeshComponent* RawSkeletalMeshComponent = SkeletalMeshComponent.Get();
	if (!RawSkeletalMeshComponent)
	{
		return FTransform::Identity;
	}

	USceneComponent* Parent = RawSkeletalMeshComponent->GetAttachParent();
	FName AttachSocket = RawSkeletalMeshComponent->GetAttachSocketName();
	if (!Parent)
	{
		return GetRelativeTransform(InSocketName);
	}

	FTransform Global = GetRelativeTransform(InSocketName);
	while (Parent)
	{
		if (USkeletalMeshComponent* ParentSkeletalMeshComponent = Cast<USkeletalMeshComponent>(Parent))
		{
			const FAnimationEvaluator& ParentEvaluator = FAnimationEvaluationCache::Get().GetEvaluator(ParentSkeletalMeshComponent);
			Global = Global * ParentEvaluator.GetRelativeTransform(AttachSocket);
		}
		else
		{
			Global = Global * Parent->GetRelativeTransform();	
		}
		
		AttachSocket = Parent->GetAttachSocketName();
		Parent = Parent->GetAttachParent();
	}

	return Global;
}

FTransform FAnimationEvaluator::GetRelativeTransform(const FName InSocketName) const
{
	USkeletalMeshComponent* RawSkeletalMeshComponent = SkeletalMeshComponent.Get();
	if (!RawSkeletalMeshComponent)
	{
		return FTransform::Identity;
	}

	
	USkeletalMesh* SkeletalMesh = IsValid() ? Context.SkeletalMesh : RawSkeletalMeshComponent->GetSkeletalMeshAsset();
	auto GetSocketTransform = [InSocketName, RawSkeletalMeshComponent, SkeletalMesh](const TArray<FTransform>& ComponentSpaceTransforms)
	{
		FTransform SocketTransform;
		int32 BoneIndex = INDEX_NONE;

		if (RawSkeletalMeshComponent->GetSocketInfoByName(InSocketName, SocketTransform, BoneIndex))
		{
			if (ensure(BoneIndex != INDEX_NONE && ComponentSpaceTransforms.IsValidIndex(BoneIndex)))
			{
				return SocketTransform * ComponentSpaceTransforms[BoneIndex];
			}
		}

		if (SkeletalMesh)
		{
			BoneIndex = SkeletalMesh->GetRefSkeleton().FindBoneIndex(InSocketName);
			if (ensure(BoneIndex != INDEX_NONE && ComponentSpaceTransforms.IsValidIndex(BoneIndex)))
			{
				return ComponentSpaceTransforms[BoneIndex];
			}
		}

		return FTransform::Identity;
	};
	
	const FTransform& RelativeTransform = RawSkeletalMeshComponent->GetRelativeTransform();

	constexpr bool bValidForTransforms = true;
	const TArray<FTransform>& ComponentSpaceTransforms = IsValid(bValidForTransforms) ?
		Context.ComponentSpaceTransforms : RawSkeletalMeshComponent->GetComponentSpaceTransforms();
	
	return InSocketName == NAME_None ? RelativeTransform : GetSocketTransform(ComponentSpaceTransforms) * RelativeTransform;
}

bool FAnimationEvaluator::AddPostEvaluationTask(const FAnimationEvaluationTask& InTask)
{
	if (USkeletalMeshComponent* RawSkeletalMeshComponent = SkeletalMeshComponent.Get())
	{
		if (InTask.IsValid(RawSkeletalMeshComponent) && !PostEvaluationTasks.Contains(InTask.Guid))
		{
			PostEvaluationTasks.FindOrAdd(InTask.Guid, InTask);
			return true;
		}
	}
	
	return false;
}
	
void FAnimationEvaluator::BindDelegates()
{
	UnbindDelegates();

	if (USkeletalMeshComponent* RawSkeletalMeshComponent = SkeletalMeshComponent.Get())
	{
		const FOnBoneTransformsFinalizedMultiCast::FDelegate OnBoneTransformsFinalizedDelegate =
			FOnBoneTransformsFinalizedMultiCast::FDelegate::CreateRaw(this, &FAnimationEvaluator::BoneTransformsFinalized);
		OnBoneTransformsFinalizedHandle = RawSkeletalMeshComponent->RegisterOnBoneTransformsFinalizedDelegate(OnBoneTransformsFinalizedDelegate);
		
		if (bLogNewEvaluation)
		{
			UE_LOGF(LogTemp, Warning, "[%p] FAnimationEvaluator register finalize bone from %ls", this, *GetComponentName(RawSkeletalMeshComponent));
		}
	}
}
	
void FAnimationEvaluator::UnbindDelegates()
{
	if (OnBoneTransformsFinalizedHandle.IsValid())
	{
		constexpr bool bEvenIfPendingKill = true;
		if (USkeletalMeshComponent* RawSkeletalMeshComponent = SkeletalMeshComponent.Get(bEvenIfPendingKill))
		{
			RawSkeletalMeshComponent->UnregisterOnBoneTransformsFinalizedDelegate(OnBoneTransformsFinalizedHandle);
			if (bLogNewEvaluation)
			{
				UE_LOGF(LogTemp, Warning, "[%p]  FAnimationEvaluator unregister finalize bone from %ls", this, *GetComponentName(RawSkeletalMeshComponent));
			}
		}
		OnBoneTransformsFinalizedHandle.Reset();
	}
}
	
void FAnimationEvaluator::UpdateContext()
{
	if (!SkeletalMeshComponent.IsValid())
	{
		Context.Clear();
		return;
	}

	const bool bEvaluatePostProcessInstance = SkeletalMeshComponent->ShouldEvaluatePostProcessInstance();
	
	Context.SkeletalMesh = SkeletalMeshComponent->GetSkeletalMeshAsset();
	Context.AnimInstance = SkeletalMeshComponent->AnimScriptInstance;
	Context.PostProcessAnimInstance = bEvaluatePostProcessInstance? ToRawPtr(SkeletalMeshComponent->PostProcessAnimInstance): nullptr;
	Context.bDoEvaluation = true;
	Context.bDoInterpolation = false;
	Context.bDuplicateToCacheBones = false;
	Context.bDuplicateToCacheCurve = false;
	Context.bDuplicateToCachedAttributes = false;
	Context.bForceRefPose = false;
}
	
void FAnimationEvaluator::RefreshBoneTransforms()
{
	// see. USkeletalMeshComponent::RefreshBoneTransforms
	if (!SkeletalMeshComponent.IsValid())
	{
		return;
	}

	// avoid re-entrant animation evaluation
	if (SkeletalMeshComponent->IsPostEvaluatingAnimation())
	{
		return;
	}

	constexpr bool bBlockOnTask = true, bPerformPostAnimEvaluation = true;
	if (SkeletalMeshComponent->HandleExistingParallelEvaluationTask(bBlockOnTask, bPerformPostAnimEvaluation))
	{
		return;
	}

	// disable UpdateRateOptimizations
	using UROGuard = TGuardValue_Bitfield_Cleanup<TFunction<void()>>;
	UROGuard Guard([URORefValue = SkeletalMeshComponent->bEnableUpdateRateOptimizations, this]()
	{
		SkeletalMeshComponent->bEnableUpdateRateOptimizations = URORefValue;
	});
	SkeletalMeshComponent->bEnableUpdateRateOptimizations = false;

	// update context & evaluate
	UpdateContext();
	EvaluateAnimation();
}

void FAnimationEvaluator::EvaluateAnimation()
{
	USkeletalMeshComponent* RawSkeletalMeshComponent = SkeletalMeshComponent.Get();
	if (!RawSkeletalMeshComponent)
	{
		return;
	}
	
	// see USkeletalMeshComponent:DoInstancePreEvaluation()
	{
		if (Context.AnimInstance)
		{
			Context.AnimInstance->PreEvaluateAnimation();
		}

		if (Context.PostProcessAnimInstance)
		{
			Context.PostProcessAnimInstance->PreEvaluateAnimation();
		}
	}
	
	// call USkeletalMeshComponent::TickAnimation() if needed
	{
		bool bShouldTickAnimation = false;
		
		if (Context.AnimInstance && !Context.AnimInstance->NeedsUpdate())
		{
			bShouldTickAnimation = !Context.AnimInstance->GetUpdateCounter().HasEverBeenUpdated();
		}

		const bool bBPEnabled = !RawSkeletalMeshComponent->GetDisablePostProcessBlueprint() && RawSkeletalMeshComponent->ShouldEvaluatePostProcessAnimBP();
		if (bBPEnabled && Context.PostProcessAnimInstance && !Context.PostProcessAnimInstance->NeedsUpdate())
		{
			bShouldTickAnimation |= !Context.PostProcessAnimInstance->GetUpdateCounter().HasEverBeenUpdated();
		}

		if (bShouldTickAnimation || IsCaching())
		{
			// We bypass TickPose() and call TickAnimation directly, so URO doesn't intercept us.
			auto GetDeltaTime = [&CachedInOut = CachedForDeltaTime, ToEvaluateForDeltaTime = RawSkeletalMeshComponent]()
			{
				constexpr float DeltaTime = 0.0f;
				if (!IsCaching())
				{
					return DeltaTime;
				}
				
				bool bAlreadyContained = false;
				{
					TScopeLock ScopeLock(CachingLock);
					CachedInOut.FindOrAdd(ToEvaluateForDeltaTime, &bAlreadyContained);
				}
				return bAlreadyContained ? DeltaTime : CachingDelta;
			};
			static constexpr bool bNeedsValidRootMotion = false;
		
			RawSkeletalMeshComponent->TickAnimation(GetDeltaTime(), bNeedsValidRootMotion);
		}
	}

	// see USkeletalMeshComponent::DoParallelEvaluationTasks_OnGameThread()
	{
		Context.ComponentSpaceTransforms = RawSkeletalMeshComponent->GetEditableComponentSpaceTransforms();
		Context.CachedComponentSpaceTransforms = RawSkeletalMeshComponent->GetCachedComponentSpaceTransforms();
		Context.BoneSpaceTransforms = RawSkeletalMeshComponent->GetBoneSpaceTransforms();
		Context.Curve = RawSkeletalMeshComponent->GetAnimCurves();
		Context.RootBoneTranslation = RawSkeletalMeshComponent->RootBoneTranslation;

		// note that don't use curves and custom attributes here 
	}
		
	// see USkeletalMeshComponent::ParallelAnimationEvaluation()
	{
		RawSkeletalMeshComponent->PerformAnimationProcessing(
		   Context.SkeletalMesh,
		   Context.AnimInstance,
		   Context.bDoEvaluation,
		   Context.bForceRefPose,
		   Context.ComponentSpaceTransforms,
		   Context.BoneSpaceTransforms,
		   Context.RootBoneTranslation,
		   Context.Curve,
		   Context.CustomAttributes);
	}

	// call post evaluation tasks if any
	for (auto PostEvaluationTaskIt = PostEvaluationTasks.CreateIterator(); PostEvaluationTaskIt; ++PostEvaluationTaskIt)
	{
		const FAnimationEvaluationTask& PostEvaluationTask = PostEvaluationTaskIt.Value();
		if (!PostEvaluationTask.IsValid(RawSkeletalMeshComponent))
		{
			PostEvaluationTaskIt.RemoveCurrent();
		}
		else
		{
			if (bLogNewEvaluation)
			{
				UE_LOGF(LogTemp, Warning, "EvaluateTask %ls", *PostEvaluationTask.Guid.ToString() );
			}
			PostEvaluationTask.PostEvaluationFunction();
		}
	}
}

void FAnimationEvaluator::BoneTransformsFinalized()
{
	if (!IsValid())
	{
		return;
	}

	USkeletalMeshComponent* RawSkeletalMeshComponent = SkeletalMeshComponent.Get();

	bool bFinalized = false;

	// NOTE FAnimationEvaluator could store a TBitArray of bones being requested and only check them instead of checking all of them 
	const TArrayView<const FTransform> BonesTransforms = RawSkeletalMeshComponent->GetComponentSpaceTransforms();
	if (Context.ComponentSpaceTransforms.Num() == BonesTransforms.Num())
	{
		for (int32 Index = 0; Index < BonesTransforms.Num(); ++Index)
		{
			FTransform& ContextBoneTransform = Context.ComponentSpaceTransforms[Index];
			if (!ContextBoneTransform.Equals(BonesTransforms[Index]))
			{
				ContextBoneTransform = BonesTransforms[Index];
				bFinalized = true;
			}
		}
	}

	// call post evaluation tasks if any
	for (auto PostEvaluationTaskIt = PostEvaluationTasks.CreateIterator(); PostEvaluationTaskIt; ++PostEvaluationTaskIt)
	{
		const FAnimationEvaluationTask& PostEvaluationTask = PostEvaluationTaskIt.Value();
		if (!PostEvaluationTask.IsValid(RawSkeletalMeshComponent))
		{
			PostEvaluationTaskIt.RemoveCurrent();
		}
		else
		{
			PostEvaluationTask.PostEvaluationFunction();
		}
	}
}
	
FAnimationEvaluationCache& FAnimationEvaluationCache::Get()
{
	static FAnimationEvaluationCache AnimationEvaluationCache;
	if (!AnimationEvaluationCache.ConstraintsNotificationHandle.IsValid())
	{
		AnimationEvaluationCache.RegisterNotifications();
	}
	return AnimationEvaluationCache;
}

FAnimationEvaluationCache::~FAnimationEvaluationCache()
{
	UnregisterNotifications();
	PerSkeletalMeshEvaluator.Reset();
}
	
void FAnimationEvaluationCache::MarkForEvaluation(const USkeletalMeshComponent* InSkeletalMeshComponent)
{
	if (!InSkeletalMeshComponent)
	{
		return;
	}

	if (TUniquePtr<FAnimationEvaluator>* Found = PerSkeletalMeshEvaluator.Find(InSkeletalMeshComponent))
	{
		FAnimationEvaluator* AnimationEvaluator = Found->Get();
		if (ensure(AnimationEvaluator))
		{
			if (AnimationEvaluator->IsValid())
			{
				if (bLogNewEvaluation)
				{
					UE_LOGF(LogTemp, Warning, "Marked %ls's evaluator for evaluation.", *GetComponentName(InSkeletalMeshComponent));
				}
				constexpr bool bDoNotRefreshBones = false;
				AnimationEvaluator->Update(bDoNotRefreshBones);
			}
		}
	}
}
	
const FAnimationEvaluator& FAnimationEvaluationCache::GetEvaluator(USkeletalMeshComponent* InSkeletalMeshComponent)
{
	if (!InSkeletalMeshComponent)
	{
		static const FAnimationEvaluator Invalid(nullptr);
		return Invalid;
	}
	
	constexpr bool bRefreshBones = true;

	if (TUniquePtr<FAnimationEvaluator>* Found = PerSkeletalMeshEvaluator.Find(InSkeletalMeshComponent))
	{
		FAnimationEvaluator* AnimationEvaluator = Found->Get();
		if (!ensure(AnimationEvaluator))
		{
			(*Found) = MakeUnique<FAnimationEvaluator>(InSkeletalMeshComponent);
			AnimationEvaluator = Found->Get();
		}
		
		if (!AnimationEvaluator->IsValid())
		{
			if (bLogNewEvaluation)
			{
				UE_LOGF(LogTemp, Warning, "Update %ls's evaluator for evaluation.", *GetComponentName(InSkeletalMeshComponent));
			}
			AnimationEvaluator->Update(bRefreshBones);
		}
		return *AnimationEvaluator;
	}
	
	TUniquePtr<FAnimationEvaluator>& NewEvaluator = PerSkeletalMeshEvaluator.Emplace(InSkeletalMeshComponent, new FAnimationEvaluator(InSkeletalMeshComponent));
	NewEvaluator.Get()->Update(bRefreshBones);

	if (bLogNewEvaluation)
	{
		UE_LOGF(LogTemp, Warning, "Create new evaluator for %ls.", *GetComponentName(InSkeletalMeshComponent));
	}
	
	return *NewEvaluator.Get();
}

const FAnimationEvaluator& FAnimationEvaluationCache::GetEvaluator(USkeletalMeshComponent* InSkeletalMeshComponent,
	const FAnimationEvaluationTask& InTask)
{
	if (!InSkeletalMeshComponent)
	{
		static const FAnimationEvaluator Invalid(nullptr);
		return Invalid;
	}
	
	constexpr bool bRefreshBones = true;

	if (TUniquePtr<FAnimationEvaluator>* Found = PerSkeletalMeshEvaluator.Find(InSkeletalMeshComponent))
	{
		FAnimationEvaluator* AnimationEvaluator = Found->Get();
		if (!ensure(AnimationEvaluator))
		{
			(*Found) = MakeUnique<FAnimationEvaluator>(InSkeletalMeshComponent);
			AnimationEvaluator = Found->Get();
		}
		
		const bool bAdded = AnimationEvaluator->AddPostEvaluationTask(InTask);
		if (!AnimationEvaluator->IsValid())
		{
			if (bLogNewEvaluation)
			{
				UE_LOGF(LogTemp, Warning, "Update %ls's evaluator for evaluation.", *GetComponentName(InSkeletalMeshComponent));
			}
			AnimationEvaluator->Update(bRefreshBones);
		}
		else if (bAdded && InTask.IsValid(InSkeletalMeshComponent))
		{
			InTask.PostEvaluationFunction();
		}
		return *AnimationEvaluator;
	}
	
	TUniquePtr<FAnimationEvaluator>& NewEvaluator = PerSkeletalMeshEvaluator.Emplace(InSkeletalMeshComponent, new FAnimationEvaluator(InSkeletalMeshComponent));
	NewEvaluator.Get()->AddPostEvaluationTask(InTask);
	NewEvaluator.Get()->Update(bRefreshBones);

	if (bLogNewEvaluation)
	{
		UE_LOGF(LogTemp, Warning, "Create new evaluator for %ls.", *GetComponentName(InSkeletalMeshComponent));
	}
	
	return *NewEvaluator.Get();
}

const FAnimationEvaluator& FAnimationEvaluationCache::FindEvaluator(const USkeletalMeshComponent* InSkeletalMeshComponent) const
{
	if (const TUniquePtr<FAnimationEvaluator>* Found = InSkeletalMeshComponent ? PerSkeletalMeshEvaluator.Find(InSkeletalMeshComponent) : nullptr)
	{
		if (const FAnimationEvaluator* AnimationEvaluator = Found->Get())
		{
			return *AnimationEvaluator;	
		}
	}
	
	static const FAnimationEvaluator Invalid(nullptr);
	return Invalid;
}
	
void FAnimationEvaluationCache::RegisterNotifications()
{
	if (!ConstraintsNotificationHandle.IsValid())
	{
		ConstraintsNotificationHandle =
		   FConstraintsManagerController::GetNotifyDelegate().AddLambda([this](EConstraintsManagerNotifyType InNotifyType, UObject* InObject)
	   {
		   if (InNotifyType == EConstraintsManagerNotifyType::GraphUpdated)
		   {
			   PerSkeletalMeshEvaluator.Reset();
		   }
	   });
	}
}

void FAnimationEvaluationCache::UnregisterNotifications()
{
	if (ConstraintsNotificationHandle.IsValid())
	{
		FConstraintsManagerController::GetNotifyDelegate().Remove(ConstraintsNotificationHandle);
		ConstraintsNotificationHandle.Reset();
	}
}

void MarkComponentForEvaluation(const USceneComponent* InSceneComponent)
{
	if (const USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(InSceneComponent))
	{
		FAnimationEvaluationCache::Get().MarkForEvaluation(SkeletalMeshComponent);
	}
}

const FAnimationEvaluator& EvaluateComponent(USceneComponent* InSceneComponent)
{
	if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(InSceneComponent))
	{
		return FAnimationEvaluationCache::Get().GetEvaluator(SkeletalMeshComponent);
	}

	static const FAnimationEvaluator Invalid(nullptr);
	return Invalid;
}

}
