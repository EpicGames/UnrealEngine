// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transform/DependencyBuilder.h"

#include "ConstraintsManager.inl"

#include "Transform/TransformableHandle.h"
#include "Transform/TransformableRegistry.h"
#include "Transform/TransformConstraint.h"
#include "Transform/TransformConstraintUtil.h"

namespace DependencyLocals
{
	
uint32 GetConstrainableHash(const UObject* InObject)
{
	// look for customized hash function
	const FTransformableRegistry& Registry = FTransformableRegistry::Get();
	if (const FTransformableRegistry::GetHashFuncT HashFunction = Registry.GetHashFunction(InObject->GetClass()))
	{
		return HashFunction(InObject, NAME_None);
	}

	return 0;
}
	
UObject* GetHandleTarget(const TObjectPtr<UTransformableHandle>& InHandle)
{
	return IsValid(InHandle) ? InHandle->GetTarget().Get() : nullptr; 
}

static bool	bDebugDependencies = false;
static FAutoConsoleVariableRef CVarDebugDependencies(
	TEXT("Constraints.DebugDependencies"),
	bDebugDependencies,
	TEXT("Print debug info about dependencies when creating a new constraint.") );

static bool	bSupportIndirectDependencies = false;
static FAutoConsoleVariableRef CVarSupportIndirectDependencies(
	TEXT("Constraints.SupportIndirectDependencies"),
	bSupportIndirectDependencies,
	TEXT("Support self indirect dependencies between handles.") );

FString GetConstraintLabel(const UTickableConstraint* InConstraint)
{
#if WITH_EDITOR
	return InConstraint->GetFullLabel();
#else
	return InConstraint->GetName();
#endif		
}

FString GetHandleLabel(const UTransformableHandle* InHandle)
{
#if WITH_EDITOR
	return InHandle->GetFullLabel();
#else
	return InHandle->GetName();
#endif		
}

void LogDependency( const FString& InDescription, const UTransformableHandle* InParentHandle, const UTransformableHandle* InChildHandle,
					const UTickableConstraint* InParentConstraint, const UTickableConstraint* InChildConstraint)
{
	if (!bDebugDependencies)
	{
		return;
	}
	
	if (!InParentHandle || !InChildHandle || !InParentConstraint || !InChildConstraint)
	{
		return;
	}

	UE_LOGF(LogTemp, Warning, "%ls: '%ls' is parent of '%ls' so '%ls' must tick before '%ls'",
		*InDescription,
		*GetHandleLabel(InParentHandle), *GetHandleLabel(InChildHandle),
		*GetConstraintLabel(InParentConstraint), *GetConstraintLabel(InChildConstraint));
}
	
void LogIndirectDependency(const UTransformableHandle* InParentHandle, const UTransformableHandle* InChildHandle,
	const UTickableConstraint* InParentConstraint, const UTickableConstraint* InChildConstraint)
{
	if (!bDebugDependencies)
	{
		return;
	}
	
	if (!InParentHandle || !InChildHandle || !InParentConstraint || !InChildConstraint)
	{
		return;
	}

	UE_LOGF(LogTemp, Warning, "Self Indirect Dependency: '%ls' has a lower hierarchy depth compared to '%ls' so '%ls' must tick before '%ls'",
		*GetHandleLabel(InParentHandle), *GetHandleLabel(InChildHandle),
		*GetConstraintLabel(InParentConstraint), *GetConstraintLabel(InChildConstraint));
}

void LogAttachmentDependency( const FString& InDescription, const UTransformableHandle* InAttachHandle, const UTransformableHandle* InChildHandle,
								const UTickableConstraint* InAttachConstraint, const UTickableConstraint* InChildConstraint)
{
	if (!bDebugDependencies)
	{
		return;
	}
	
	if (!InAttachHandle || !InChildHandle || !InAttachConstraint || !InChildConstraint)
	{
		return;
	}

	UE_LOGF(LogTemp, Warning, "%ls: '%ls' is an attach parent of '%ls' so '%ls' must tick before '%ls'",
		*InDescription,
		*GetHandleLabel(InAttachHandle), *GetHandleLabel(InChildHandle),
		*GetConstraintLabel(InAttachConstraint), *GetConstraintLabel(InChildConstraint));
}

// we suppose that both InParentHandle and InChildHandle are safe to use
bool HasConstraintDependencyWith(UWorld* InWorld, const UTransformableHandle* InParentHandle, const UTransformableHandle* InChildHandle)
{
	using ConstraintPtr = TWeakObjectPtr<UTickableConstraint>;
	using HandlePtr = TObjectPtr<UTransformableHandle>;

	static constexpr bool bSorted = false;
	const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(InWorld);
	const TArray<ConstraintPtr> Constraints = Controller.GetParentConstraints(InParentHandle->GetHash(), bSorted);

	// get parent handles
	TArray< HandlePtr > ParentHandles;
	for (const ConstraintPtr& Constraint: Constraints)
	{
		if (const UTickableTransformConstraint* TransformConstraint = Cast<UTickableTransformConstraint>(Constraint.Get()))
		{
			if (IsValid(TransformConstraint->ParentTRSHandle))
			{
				ParentHandles.Add(TransformConstraint->ParentTRSHandle);
			}
		}
	}

	// check if InChildHandle is one of them
	const uint32 ChildHash = InChildHandle->GetHash();
	const bool bIsParentADependency = ParentHandles.ContainsByPredicate([ChildHash](const HandlePtr& InHandle)
	{
		return InHandle->GetHash() == ChildHash;
	});

	if (bIsParentADependency)
	{
		return true;
	}

	// if not, recurse
	for (const HandlePtr& ParentHandle: ParentHandles)
	{
		if (HasConstraintDependencyWith(InWorld, ParentHandle, InChildHandle))
		{
			return true;
		}
	}

	return false;
}	
	
}

bool FDependencyBuilder::LogDependencies()
{
	return DependencyLocals::bDebugDependencies;
}

void FDependencyBuilder::BuildSelfDependencies(UWorld* InWorld, UTickableTransformConstraint* InConstraint)
{
	using namespace DependencyLocals;
	using ConstraintWeakPtr = TWeakObjectPtr<UTickableConstraint>;

	static const TCHAR* SelfDependencyDesc = TEXT("Self Dependency");
	
	if (!InConstraint || !InConstraint->IsValid())
	{
		return;
	}
	
	const UTransformableHandle* ParentHandle = InConstraint->ParentTRSHandle.Get();
	const UTransformableHandle* ChildHandle = InConstraint->ChildTRSHandle.Get();
		
	const UObject* ParentTarget = GetHandleTarget(InConstraint->ParentTRSHandle);
	const UObject* ChildTarget = GetHandleTarget(InConstraint->ChildTRSHandle);
	
	const bool bSelf = ParentTarget && ParentTarget == ChildTarget;
	if (!bSelf)
	{
		return;
	}

	const UObject* SelfTarget = ChildTarget;

	auto SetDependency = [InWorld](const FGuid& InGuidToTickBefore, const FGuid& InGuidToTickAfter)
	{
		FConstraintsManagerController& Controller = FConstraintsManagerController::Get(InWorld);
		Controller.SetConstraintsDependencies(InGuidToTickBefore, InGuidToTickAfter);
	};
	
	auto SelfTargetPredicate = [InConstraint, SelfTarget](const ConstraintWeakPtr& Constraint)
	{
		const UTickableTransformConstraint* TransformConstraint = Cast<UTickableTransformConstraint>(Constraint.Get());
		if (!TransformConstraint || TransformConstraint == InConstraint)
		{
			return false;
		}
		const UObject* ParentTarget = GetHandleTarget(TransformConstraint->ParentTRSHandle);
		const UObject* ChildTarget = GetHandleTarget(TransformConstraint->ChildTRSHandle);
		return ParentTarget == SelfTarget && ChildTarget == SelfTarget;
	};
	const TArray< ConstraintWeakPtr > SelfConstraints = FConstraintsManagerController::Get(InWorld).GetConstraintsByPredicate(SelfTargetPredicate);

	bool bDirectDependencyAdded = false;
	for (const ConstraintWeakPtr& SelfConstraint: SelfConstraints)
	{
		const UTickableTransformConstraint* TransformConstraint = Cast<UTickableTransformConstraint>(SelfConstraint);

		// if the new handles depend on that constraint child then, TransformConstraint should tick before
		if (ParentHandle->HasDirectDependencyWith(*TransformConstraint->ChildTRSHandle))
		{
			SetDependency(TransformConstraint->ConstraintID, InConstraint->ConstraintID);
			LogDependency(SelfDependencyDesc, TransformConstraint->ChildTRSHandle, ParentHandle, TransformConstraint, InConstraint);
			bDirectDependencyAdded = true;
		}
		else if (ChildHandle->HasDirectDependencyWith(*TransformConstraint->ChildTRSHandle))
		{
			SetDependency(TransformConstraint->ConstraintID, InConstraint->ConstraintID);
			LogDependency(SelfDependencyDesc, TransformConstraint->ChildTRSHandle, ChildHandle, TransformConstraint, InConstraint);
			bDirectDependencyAdded = true;
		}

		// if the TransformConstraint handles depend on the new constraint child then, TransformConstraint should tick after
		if (TransformConstraint->ParentTRSHandle->HasDirectDependencyWith(*ChildHandle))
		{
			SetDependency(InConstraint->ConstraintID, TransformConstraint->ConstraintID);
			LogDependency(SelfDependencyDesc, ChildHandle, TransformConstraint->ParentTRSHandle, InConstraint, TransformConstraint);
			bDirectDependencyAdded = true;
		}
		else if (TransformConstraint->ChildTRSHandle->HasDirectDependencyWith(*ChildHandle))
		{
			SetDependency(InConstraint->ConstraintID, TransformConstraint->ConstraintID);
			LogDependency(SelfDependencyDesc, ChildHandle, TransformConstraint->ChildTRSHandle, InConstraint, TransformConstraint);
			bDirectDependencyAdded = true;
		}
	}

	// look for indirect dependencies (hierarchy depth, for example)
	if (bSupportIndirectDependencies && !bDirectDependencyAdded)
	{
		for (const ConstraintWeakPtr& SelfConstraint: SelfConstraints)
		{
			const UTickableTransformConstraint* TransformConstraint = Cast<UTickableTransformConstraint>(SelfConstraint);
			
			// if the new handles indirectly depend on that constraint child then, TransformConstraint should tick before
			if (ParentHandle->HasIndirectDependencyWith(*TransformConstraint->ChildTRSHandle))
			{
				SetDependency(TransformConstraint->ConstraintID, InConstraint->ConstraintID);
				LogIndirectDependency(TransformConstraint->ChildTRSHandle, ParentHandle, TransformConstraint, InConstraint);
			}
			else if (ChildHandle->HasIndirectDependencyWith(*TransformConstraint->ChildTRSHandle))
			{
				SetDependency(TransformConstraint->ConstraintID, InConstraint->ConstraintID);
				LogIndirectDependency(TransformConstraint->ChildTRSHandle, ChildHandle, TransformConstraint, InConstraint);
			}

			// if the TransformConstraint handles indirectly depend on the new constraint child then, TransformConstraint should tick after
			if (TransformConstraint->ParentTRSHandle->HasIndirectDependencyWith(*ChildHandle))
			{
				SetDependency(InConstraint->ConstraintID, TransformConstraint->ConstraintID);
				LogIndirectDependency(ChildHandle, TransformConstraint->ParentTRSHandle, InConstraint, TransformConstraint);
			}
			else if (TransformConstraint->ChildTRSHandle->HasIndirectDependencyWith(*ChildHandle))
			{
				SetDependency(InConstraint->ConstraintID, TransformConstraint->ConstraintID);
				LogIndirectDependency(ChildHandle, TransformConstraint->ChildTRSHandle, InConstraint, TransformConstraint);
			}
		}
	}
}

void FDependencyBuilder::BuildExternalDependencies(UWorld* InWorld, UTickableTransformConstraint* InConstraint)
{
	using namespace DependencyLocals;
	using ConstraintWeakPtr = TWeakObjectPtr<UTickableConstraint>;

	if (!InConstraint || !InConstraint->IsValid())
	{
		return;
	}
	
	const UTransformableHandle* ParentHandle = InConstraint->ParentTRSHandle.Get();
	const UTransformableHandle* ChildHandle = InConstraint->ChildTRSHandle.Get();
		
	const UObject* ParentTarget = GetHandleTarget(InConstraint->ParentTRSHandle);
	const UObject* ChildTarget = GetHandleTarget(InConstraint->ChildTRSHandle);
	
	const bool bSelf = ParentTarget && ParentTarget == ChildTarget;
	if (bSelf)
	{
		return;
	}
	
	{
		// get all constraints acting on the same target 
		auto SameChildTargetPredicate = [ChildTarget](const ConstraintWeakPtr& Constraint)
		{
			const UTickableTransformConstraint* TransformConstraint = Cast<UTickableTransformConstraint>(Constraint.Get());
			if (!TransformConstraint)
			{
				return false;
			}
			const UObject* OtherChildTarget = GetHandleTarget(TransformConstraint->ChildTRSHandle);
			return OtherChildTarget && OtherChildTarget == ChildTarget;
		};
		TArray< ConstraintWeakPtr > ChildTargetParentConstraints = FConstraintsManagerController::Get(InWorld).GetConstraintsByPredicate(SameChildTargetPredicate);

		// store constraint index in this array
		const int32 ConstrainIndex = ChildTargetParentConstraints.IndexOfByKey(InConstraint);
		if (ensure(ChildTargetParentConstraints.IsValidIndex(ConstrainIndex)))
		{
			ChildTargetParentConstraints.RemoveAt(ConstrainIndex);
		}
	
		if (!ChildTargetParentConstraints.IsEmpty())
		{
			static const TCHAR* ExternalDependencyDesc = TEXT("External Dependency");
		
			TBitArray<> ManagedDependencies(false, ChildTargetParentConstraints.Num());
		
			const FConstraintTickFunction& TickFunction = InConstraint->GetTickFunction(InWorld);
			const TArray<FTickPrerequisite> PrerexCopy = TickFunction.GetPrerequisites();

			for (int32 Index = 0; Index < ChildTargetParentConstraints.Num(); ++Index)
			{
				const UTickableTransformConstraint* TransformConstraint = Cast<UTickableTransformConstraint>(ChildTargetParentConstraints[Index]);
			
				// if the new handles depend on that constraint child then, TransformConstraint should tick before
				if (ParentHandle->HasDirectDependencyWith(*TransformConstraint->ChildTRSHandle))
				{
					FConstraintsManagerController::Get(InWorld).SetConstraintsDependencies(TransformConstraint->ConstraintID, InConstraint->ConstraintID);
					ManagedDependencies[Index] = true;
					LogDependency(ExternalDependencyDesc, TransformConstraint->ChildTRSHandle, ParentHandle, TransformConstraint, InConstraint);
				}
				else if (ChildHandle->HasDirectDependencyWith(*TransformConstraint->ChildTRSHandle))
				{
					FConstraintsManagerController::Get(InWorld).SetConstraintsDependencies(TransformConstraint->ConstraintID, InConstraint->ConstraintID);
					ManagedDependencies[Index] = true;
					LogDependency(ExternalDependencyDesc, TransformConstraint->ChildTRSHandle, ChildHandle, TransformConstraint, InConstraint);
				}

				// if the TransformConstraint handles depend on the new constraint child then, TransformConstraint should tick after
				if (TransformConstraint->ParentTRSHandle->HasDirectDependencyWith(*ChildHandle))
				{
					FConstraintsManagerController::Get(InWorld).SetConstraintsDependencies(InConstraint->ConstraintID, TransformConstraint->ConstraintID);
					ManagedDependencies[Index] = true;
					LogDependency(ExternalDependencyDesc, ChildHandle, TransformConstraint->ParentTRSHandle, InConstraint, TransformConstraint);
				}
				else if (TransformConstraint->ChildTRSHandle->HasDirectDependencyWith(*ChildHandle))
				{
					FConstraintsManagerController::Get(InWorld).SetConstraintsDependencies(InConstraint->ConstraintID, TransformConstraint->ConstraintID);
					ManagedDependencies[Index] = true;
					LogDependency(ExternalDependencyDesc, ChildHandle, TransformConstraint->ChildTRSHandle, InConstraint, TransformConstraint);
				}
			}

			// if we didn't add any new prerequisite then check whether the constraint should tick after the last constraint
			// acting on the same target to respect order of creation.
			const bool bPrerexChanged = TickFunction.GetPrerequisites() != PrerexCopy;
			if (!bPrerexChanged && ManagedDependencies.IsValidIndex(ConstrainIndex-1))
			{
				for (int32 Index = ConstrainIndex-1; Index >= 0; Index--)
				{
					if (!ManagedDependencies[Index])
					{
						const UTickableTransformConstraint* LastConstraintSharingSameTarget = Cast<UTickableTransformConstraint>(ChildTargetParentConstraints[Index]);
						TSet<const FTickFunction*> VisitedFunctions;
						const FTickFunction& ParentTickFunctionToCheck = LastConstraintSharingSameTarget->GetTickFunction(InWorld);
						if (!FConstraintCycleChecker::HasPrerequisiteDependencyWith(&TickFunction, &ParentTickFunctionToCheck, VisitedFunctions) &&
							!FConstraintCycleChecker::HasPrerequisiteDependencyWith(&ParentTickFunctionToCheck, &TickFunction, VisitedFunctions))
						{
							FConstraintsManagerController::Get(InWorld).SetConstraintsDependencies(LastConstraintSharingSameTarget->ConstraintID, InConstraint->ConstraintID);
							if (bDebugDependencies)
							{
								UE_LOGF(LogTemp, Warning, "Creation Order Dependency: '%ls' must tick before '%ls' to respect creation order.",
								   *GetConstraintLabel(LastConstraintSharingSameTarget), *GetConstraintLabel(InConstraint));
							}
							break;
						}
					}
				}
			}
		}
	}

	// get all constraints acting on the same target 
	{
		auto SameParentTargetPredicate = [ParentTarget](const ConstraintWeakPtr& Constraint)
		{
			const UTickableTransformConstraint* TransformConstraint = Cast<UTickableTransformConstraint>(Constraint.Get());
			if (!TransformConstraint)
			{
				return false;
			}
			const UObject* OtherChildTarget = GetHandleTarget(TransformConstraint->ChildTRSHandle);
			return OtherChildTarget && OtherChildTarget == ParentTarget;
		};
		TArray< ConstraintWeakPtr > ParentTargetParentConstraints = FConstraintsManagerController::Get(InWorld).GetConstraintsByPredicate(SameParentTargetPredicate);

		// store constraint index in this array
		const int32 ConstrainIndex = ParentTargetParentConstraints.IndexOfByKey(InConstraint);
		if (ParentTargetParentConstraints.IsValidIndex(ConstrainIndex))
		{
			ParentTargetParentConstraints.RemoveAt(ConstrainIndex);
		}

		if (!ParentTargetParentConstraints.IsEmpty())
		{
			static const TCHAR* ExternalDependencyDesc = TEXT("External Dependency");
		
			TBitArray<> ManagedDependencies(false, ParentTargetParentConstraints.Num());
		
			const FConstraintTickFunction& TickFunction = InConstraint->GetTickFunction(InWorld);
			const TArray<FTickPrerequisite> PrerexCopy = TickFunction.GetPrerequisites();

			for (int32 Index = 0; Index < ParentTargetParentConstraints.Num(); ++Index)
			{
				const UTickableTransformConstraint* TransformConstraint = Cast<UTickableTransformConstraint>(ParentTargetParentConstraints[Index]);
			
				// if the new handles depend on that constraint child then, TransformConstraint should tick before
				if (ParentHandle->HasDirectDependencyWith(*TransformConstraint->ChildTRSHandle))
				{
					FConstraintsManagerController::Get(InWorld).SetConstraintsDependencies(TransformConstraint->ConstraintID, InConstraint->ConstraintID);
					ManagedDependencies[Index] = true;
					LogDependency(ExternalDependencyDesc, TransformConstraint->ChildTRSHandle, ParentHandle, TransformConstraint, InConstraint);
				}
			}

			// if we didn't add any new prerequisite then check whether the constraint should tick after the last constraint
			// acting on the same target to respect order of creation.
			const bool bPrerexChanged = TickFunction.GetPrerequisites() != PrerexCopy;
			const int32 LastIndex = ManagedDependencies.IsValidIndex(ConstrainIndex-1) ? ConstrainIndex-1 : ParentTargetParentConstraints.Num()-1;
			if (!bPrerexChanged && ManagedDependencies.IsValidIndex(LastIndex))
			{
				for (int32 Index = LastIndex; Index >= 0; Index--)
				{
					if (!ManagedDependencies[Index])
					{
						const UTickableTransformConstraint* LastConstraintSharingSameTarget = Cast<UTickableTransformConstraint>(ParentTargetParentConstraints[Index]);
						TSet<const FTickFunction*> VisitedFunctions;
						const FTickFunction& ParentTickFunctionToCheck = LastConstraintSharingSameTarget->GetTickFunction(InWorld);
						if (!FConstraintCycleChecker::HasPrerequisiteDependencyWith(&TickFunction, &ParentTickFunctionToCheck, VisitedFunctions) &&
							!FConstraintCycleChecker::HasPrerequisiteDependencyWith(&ParentTickFunctionToCheck, &TickFunction, VisitedFunctions))
						{
							FConstraintsManagerController::Get(InWorld).SetConstraintsDependencies(LastConstraintSharingSameTarget->ConstraintID, InConstraint->ConstraintID);
							if (bDebugDependencies)
							{
								UE_LOGF(LogTemp, Warning, "Creation Order Dependency: '%ls' must tick before '%ls' to respect creation order.",
								   *GetConstraintLabel(LastConstraintSharingSameTarget), *GetConstraintLabel(InConstraint));
							}
							break;
						}
					}
				}
			}
		}
	}
	
	// get all constraints having the same parent's target as the new child's target and make sure they tick after if needed 
	{
		auto SameParentAsChildTargetPredicate = [ChildTarget, InConstraint](const ConstraintWeakPtr& Constraint)
		{
			const UTickableTransformConstraint* TransformConstraint = Cast<UTickableTransformConstraint>(Constraint.Get());
			if (!TransformConstraint || TransformConstraint == InConstraint)
			{
				return false;
			}
			const UObject* OtherParentTarget = GetHandleTarget(TransformConstraint->ParentTRSHandle);
			return OtherParentTarget && OtherParentTarget == ChildTarget;
		};
		TArray< ConstraintWeakPtr > ParentTargetChildConstraints = FConstraintsManagerController::Get(InWorld).GetConstraintsByPredicate(SameParentAsChildTargetPredicate);

		const FConstraintTickFunction& TickFunction = InConstraint->GetTickFunction(InWorld);
		for (const ConstraintWeakPtr& Constraint: ParentTargetChildConstraints)
		{
			const UTickableTransformConstraint* ConstraintSharingParentTarget = Cast<UTickableTransformConstraint>(Constraint);
			const FTickFunction& ParentTickFunctionToCheck = ConstraintSharingParentTarget->GetTickFunction(InWorld);

			TSet<const FTickFunction*> VisitedFunctions0, VisitedFunctions1;
			
			if (!FConstraintCycleChecker::HasPrerequisiteDependencyWith(&TickFunction, &ParentTickFunctionToCheck, VisitedFunctions0) &&
				!FConstraintCycleChecker::HasPrerequisiteDependencyWith(&ParentTickFunctionToCheck, &TickFunction, VisitedFunctions1))
			{
				FConstraintsManagerController::Get(InWorld).SetConstraintsDependencies(InConstraint->ConstraintID, ConstraintSharingParentTarget->ConstraintID);
				if (bDebugDependencies)
				{
					UE_LOGF(LogTemp, Warning, "External Dependency: '%ls' must tick before '%ls' as it acts on its parent.",
					   *GetConstraintLabel(InConstraint), *GetConstraintLabel(ConstraintSharingParentTarget));
				}
			}
		}
	}
}

bool FDependencyBuilder::BuildDependencies(UWorld* InWorld, UTickableTransformConstraint* InConstraint)
{
	using namespace DependencyLocals;
	using ConstraintWeakPtr = TWeakObjectPtr<UTickableConstraint>;
	
	if (!ensure(InWorld))
	{
		return false;
	}

	if (!InConstraint || !InConstraint->IsValid())
	{
		return false;
	}

	if (bDebugDependencies)
	{
		UE_LOGF(LogTemp, Warning, "Building dependencies for '%ls' ...", *GetConstraintLabel(InConstraint));
	}

	const UTransformableHandle* ParentHandle = InConstraint->ParentTRSHandle.Get();
	const UTransformableHandle* ChildHandle = InConstraint->ChildTRSHandle.Get();
	
 	// get previous child constraints
	TArray<ConstraintWeakPtr> ChildParentConstraints = FConstraintsManagerController::Get(InWorld).GetParentConstraints(ChildHandle->GetHash(), true);
	ChildParentConstraints.Remove(InConstraint);

	// add dependencies with the last child constraint
	if (!ChildParentConstraints.IsEmpty())
	{
		const FGuid LastChildConstraintID = ChildParentConstraints.Last()->ConstraintID;
		if (bDebugDependencies)
		{
			UE_LOGF(LogTemp, Warning, "Order Dependency: tick after last constraint.");
		}
		FConstraintsManagerController::Get(InWorld).SetConstraintsDependencies(LastChildConstraintID, InConstraint->ConstraintID);
	}

	const UObject* ParentTarget = ParentHandle->GetTarget().Get();
	const UObject* ChildTarget = ChildHandle->GetTarget().Get();
	const bool bSelf = ParentTarget && ParentTarget == ChildTarget;
	
	// internal dependencies?
	if (bSelf)
	{
		BuildSelfDependencies(InWorld, InConstraint);
	}

	// make sure we tick after the parent.
	InConstraint->EnsurePrimaryDependency(InWorld);

	// if child handle is the parent of some other constraints, ensure they will tick after that new one
	static const TCHAR* ChildDependencyDesc = TEXT("Child Dependency");

	TArray<ConstraintWeakPtr> ChildChildConstraints;
	UE::TransformConstraintUtil::GetChildrenConstraints(InWorld, InConstraint, ChildChildConstraints, !bSelf);
	for (const ConstraintWeakPtr& ChildConstraint: ChildChildConstraints)
	{
		FConstraintsManagerController::Get(InWorld).SetConstraintsDependencies(InConstraint->ConstraintID, ChildConstraint->ConstraintID);
		if (const UTickableTransformConstraint* TransformConstraint = Cast<UTickableTransformConstraint>(ChildConstraint))
		{
			LogDependency(ChildDependencyDesc, ChildHandle, TransformConstraint->ChildTRSHandle, InConstraint, TransformConstraint);
		}
	}

	// build dependencies regarding attachments
	BuildAttachmentsDependencies(InWorld, InConstraint);

	if (!bSelf)
	{
		BuildExternalDependencies(InWorld, InConstraint);
	}

	// warn for possible cycles
	if (FConstraintCycleChecker::IsCycling(InConstraint->ChildTRSHandle))
	{
		UE_LOGF(LogTemp, Warning, "A cycle has been formed while creating %ls.", *InConstraint->GetName());
	}

	// invalidate graph
	FConstraintsManagerController::Get(InWorld).InvalidateEvaluationGraph();
	
	return true;
}

void FDependencyBuilder::BuildAttachmentsDependencies(UWorld* InWorld, const UTickableTransformConstraint* InConstraint)
{
	using namespace DependencyLocals;
	using ConstraintWeakPtr = TWeakObjectPtr<UTickableConstraint>;
	
	if (!ensure(InWorld))
	{
		return;
	}

	if (!InConstraint || !InConstraint->IsValid())
	{
		return;
	}
	
	static const TCHAR* AttachmentDependencyDesc = TEXT("Attachment Dependency");
	
	const UTransformableHandle* ChildHandle = InConstraint->ChildTRSHandle.Get();
	if (const USceneComponent* ChildComponent = Cast<USceneComponent>(ChildHandle->GetTarget().Get()))
	{
		static constexpr bool bIncludeAllDescendants = true;
		
		TArray<USceneComponent*> ChildComponentChildren;
		ChildComponent->GetChildrenComponents(bIncludeAllDescendants, ChildComponentChildren);
		
		for (USceneComponent* ChildChildComponent: ChildComponentChildren)
		{
			const uint32 ChildHash = GetConstrainableHash(ChildChildComponent);
			if (ChildHash != 0)
			{
				auto IsHashChildOfConstraint = [ChildHash](const ConstraintWeakPtr& Constraint)
				{
					const UTickableTransformConstraint* TransformConstraint = Cast<UTickableTransformConstraint>(Constraint.Get());
					const UTransformableHandle* ChildHandle = TransformConstraint ? TransformConstraint->ChildTRSHandle.Get() : nullptr;
					if (!ChildHandle || !ChildHandle->IsValid())
					{
						return false;
					}
					return ChildHandle->GetHash() == ChildHash;
				};
				
				const TArray< ConstraintWeakPtr > AttachChildConstraints = FConstraintsManagerController::Get(InWorld).GetConstraintsByPredicate(IsHashChildOfConstraint);
				for (const ConstraintWeakPtr& AttachChildConstraint: AttachChildConstraints)
				{
					FConstraintsManagerController::Get(InWorld).SetConstraintsDependencies(InConstraint->ConstraintID, AttachChildConstraint->ConstraintID);
					if (const UTickableTransformConstraint* AttachTransformConstraint = Cast<UTickableTransformConstraint>(AttachChildConstraint))
					{
						LogAttachmentDependency(AttachmentDependencyDesc, ChildHandle, AttachTransformConstraint->ChildTRSHandle,  InConstraint, AttachTransformConstraint);
					}
				}
			}
		}
	}
}



FConstraintDependencyScope::FConstraintDependencyScope(UTickableTransformConstraint* InConstraint, UWorld* InWorld)
	: WeakConstraint(InConstraint)
	, WeakWorld(InWorld)
	, bPreviousValidity(InConstraint ? InConstraint->IsValid() : false)
{}

FConstraintDependencyScope::~FConstraintDependencyScope()
{
	if (!bPreviousValidity)
	{
		if (UTickableTransformConstraint* Constraint = WeakConstraint.IsValid() ? WeakConstraint.Get() : nullptr)
		{
			if (Constraint->IsValid())
			{
				const UObject* Target = DependencyLocals::GetHandleTarget(Constraint->ChildTRSHandle);
				UWorld* World = WeakWorld.IsValid() ? WeakWorld.Get() : Target ? Target->GetWorld() : nullptr;
				if (::IsValid(World))
				{
					FDependencyBuilder::BuildDependencies(World, Constraint);
				}
			}
		}
	}
}

FHandleDependencyChecker::FHandleDependencyChecker(UWorld* InWorld)
	: WeakWorld(InWorld)
{}

bool FHandleDependencyChecker::HasDependency(const UTransformableHandle& InHandle, const UTransformableHandle& InParentToCheck) const
{
	// check direct dependency
	if (InHandle.HasDirectDependencyWith(InParentToCheck))
	{
		return true;
	}

	UWorld* World = WeakWorld.IsValid() ? WeakWorld.Get() : nullptr;
	if (::IsValid(World))
	{
		// check constraints dependency
		if (DependencyLocals::HasConstraintDependencyWith(World, &InHandle, &InParentToCheck))
		{
			return true;
		}

		// check any existing tick dependency
		{
			TSet<const FTickFunction*> VisitedFunctions;
			const FTickFunction* TickFunction = InHandle.GetTickFunction();
			const FTickFunction* ParentTickFunctionToCheck = InParentToCheck.GetTickFunction();
			if (FConstraintCycleChecker::HasPrerequisiteDependencyWith(TickFunction, ParentTickFunctionToCheck, VisitedFunctions))
			{
				return true;
			}
		}
	}
	
	return false;
}

bool FConstraintCycleChecker::IsCycling(const TWeakObjectPtr<UTransformableHandle>& InHandle)
{
	if (!IsValid(InHandle.Get()))
	{
		return false;
	}

	TSet<const FTickFunction*> VisitedFunctions;
	const FTickFunction* TickFunction = InHandle->GetTickFunction();
	return HasPrerequisiteDependencyWith(TickFunction, TickFunction, VisitedFunctions);
}

void FConstraintCycleChecker::CheckAndFixCycles(const UTickableTransformConstraint* InConstraint)
{
	if (!IsValid(InConstraint))
	{
		return;
	}

	//todo constraints on level sequences aren't in a world
	UWorld* World = InConstraint->GetWorld();
	if (!IsValid(World))
	{
		return;
	}

	// get child's tick function
	FTickFunction* ChildTickFunction = InConstraint->GetChildHandleTickFunction();
	if (!ChildTickFunction)
	{
		return;
	}

	// filter for all constraints where the parent's tick function equals ChildTickFunction
	auto Predicate = [ChildTickFunction](const ConstraintPtr& InConstraint)
	{
		const UTickableTransformConstraint* TransformConst = Cast<UTickableTransformConstraint>(InConstraint.Get());
		if (!TransformConst)
		{
			return false;
		}

		const TObjectPtr<UTransformableHandle>& ChildHandle = TransformConst->ChildTRSHandle;
		if (!IsValid(ChildHandle) || !ChildHandle->IsValid())
		{
			return false;
		}

		const FTickFunction* ParentTickFunction = TransformConst->GetParentHandleTickFunction();
		return ParentTickFunction && ParentTickFunction == ChildTickFunction;
	};

	const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(World);
	const ConstraintArray CyclingConstraints = Controller.GetConstraintsByPredicate(Predicate);
	if (CyclingConstraints.IsEmpty())
	{
		return;
	}

	// check if they can cause a cycle and manage dependencies if that's the case
	TSet<const FTickFunction*> VisitedFunctions;
	for (const ConstraintPtr& Constraint: CyclingConstraints)
	{
		if (HasPrerequisiteDependencyWith(&Constraint->GetTickFunction(World), &InConstraint->GetTickFunction(World), VisitedFunctions))
		{
			UpdateCyclingDependency(World, Cast<UTickableTransformConstraint>(Constraint));
		}
	}
}
	
bool FConstraintCycleChecker::HasPrerequisiteDependencyWith(const FTickFunction* InSecondary, const FTickFunction* InPrimary, TSet<const FTickFunction*>& InOutVisitedFunctions)
{
	if (!InSecondary || !InPrimary)
	{
		return false;
	}

	// is InSecondary a Prereq of InPrimary?
	const TArray<FTickPrerequisite>& Prerequisites = InPrimary->GetPrerequisites();
	const bool bIsSecondaryAPrereq = Prerequisites.ContainsByPredicate([InSecondary](const FTickPrerequisite& Prereq)
	{
		const FTickFunction* PrereqFunction = Prereq.Get();
		return PrereqFunction && (PrereqFunction == InSecondary);
	});

	if (bIsSecondaryAPrereq)
	{
		return true;
	}

	// check if InPrimary has already been visited to avoid endless loop
	if (InOutVisitedFunctions.Contains(InPrimary))
	{
		return false;
	}
	InOutVisitedFunctions.Add(InPrimary);
	
	// otherwise, recurse
	for (const FTickPrerequisite& Prerequisite : Prerequisites)
	{
		if (HasPrerequisiteDependencyWith(InSecondary, Prerequisite.Get(), InOutVisitedFunctions))
		{
			return true;
		}
	}

	return false;
}

void FConstraintCycleChecker::UpdateCyclingDependency(UWorld* InWorld, UTickableTransformConstraint* InConstraintToUpdate)
{
	// nothing to do if this constraint doesn't tick
	if (!InConstraintToUpdate->GetTickFunction(InWorld).IsTickFunctionEnabled())
	{
		return;
	}
	
	const TObjectPtr<UTransformableHandle>& ParentHandle = InConstraintToUpdate->ParentTRSHandle;
	UObject* TargetObject = ParentHandle->GetPrerequisiteObject();

	// filter for all constraints where the child's target object equals ChildTickFunction
	auto Predicate = [TargetObject](const ConstraintPtr& InConstraint)
	{
		const UTickableTransformConstraint* TransformConst = Cast<UTickableTransformConstraint>(InConstraint.Get());
		if (!TransformConst)
		{
			return false;
		}

		const TObjectPtr<UTransformableHandle>& ChildHandle = TransformConst->ChildTRSHandle;
		if (!IsValid(ChildHandle) || !ChildHandle->IsValid())
		{
			return false;
		}
		
		const UObject* ChildPrereqObject = ChildHandle->GetPrerequisiteObject();
		return IsValid(ChildPrereqObject) && (ChildPrereqObject == TargetObject);
	};

	const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(InWorld);
	const ConstraintArray ParentConstraints = Controller.GetConstraintsByPredicate(Predicate, true);

	// check if there's any active constraint in ParentConstraints
	const bool bHasActiveParentConstraint = ParentConstraints.ContainsByPredicate([](const ConstraintPtr& Constraint)
	{
		return Constraint.IsValid() && IsValid(Constraint.Get()) && Constraint->IsFullyActive(); 
	});

	// update the constraint prerequisites based on the result
	if (FTickFunction* TargetTickFunction = InConstraintToUpdate->GetParentHandleTickFunction())
	{
		if (bHasActiveParentConstraint)
		{
			InConstraintToUpdate->GetTickFunction(InWorld).RemovePrerequisite(TargetObject, *TargetTickFunction);
		}
		else
		{
			InConstraintToUpdate->GetTickFunction(InWorld).AddPrerequisite(TargetObject, *TargetTickFunction);
		}
	}
}