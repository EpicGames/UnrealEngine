// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transform/TransformConstraint.h"

#include "Components/SkeletalMeshComponent.h"
#include "ConstraintsManager.inl"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "MovieSceneSection.h"
#include "Transform/TransformableHandle.h"
#include "Transform/TransformableRegistry.h"
#include "Transform/DependencyBuilder.h"
#include "Transform/TransformableHandleUtils.h"
#include "Transform/TransformConstraintUtil.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TransformConstraint)

namespace UE::Private
{

bool ShouldForceChildDependency()
{
	return TransformableHandleUtils::SkipTicking();
}

bool ShouldForceParentDependency()
{
	return !TransformableHandleUtils::SkipTicking();
}
	
bool ShouldPreEvaluateChild()
{
	return !TransformableHandleUtils::SkipTicking();
}

bool ShouldPreEvaluateParent()
{
	return TransformableHandleUtils::SkipTicking();
}
	
void PreEvaluateChild(const TObjectPtr<UTransformableHandle>& InHandle) 
{
	if (ShouldPreEvaluateChild() && ::IsValid(InHandle))
	{
		constexpr bool bTickChild = false;
		InHandle->PreEvaluate(bTickChild);
	}
}

void PreEvaluateParent(const TObjectPtr<UTransformableHandle>& InHandle) 
{
	if (ShouldPreEvaluateParent() && ::IsValid(InHandle))
	{
		constexpr bool bTickParent = false;
		InHandle->PreEvaluate(bTickParent);
	}
}

UObject* GetHandleTarget(const TObjectPtr<UTransformableHandle>& InHandle)
{
	return IsValid(InHandle) ? InHandle->GetTarget().Get() : nullptr; 
}

UTransformableHandle* GetHandle(UObject* InObject, const FName& InSocketName)
{
	// look for customized transform handle
	const FTransformableRegistry& Registry = FTransformableRegistry::Get();
	if (const FTransformableRegistry::CreateHandleFuncT CreateFunction = Registry.GetCreateFunction(InObject->GetClass()))
	{
		return CreateFunction(InObject, InSocketName);
	}

	return nullptr;
}
	
uint32 GetConstrainableHash(const UObject* InObject)
{
	// look for customized hash function
	const FTransformableRegistry& Registry = FTransformableRegistry::Get();
	if (const FTransformableRegistry::GetHashFuncT HashFunction = Registry.GetHashFunction(InObject->GetClass()))
	{
		return HashFunction(InObject, NAME_None);
	}

	return 0;
}

FString GetConstraintLabel(const UTickableConstraint* InConstraint)
{
#if WITH_EDITOR
	return InConstraint->GetFullLabel();
#else
	return InConstraint->GetName();
#endif		
}
	
}

/** 
 * UTickableTransformConstraint
 **/

#if WITH_EDITOR
UTickableTransformConstraint::FOnConstraintChanged UTickableTransformConstraint::OnConstraintChanged;
#endif // WITH_EDITOR

int64 UTickableTransformConstraint::GetType() const
{
	return static_cast<int64>(Type);
}
EMovieSceneTransformChannel UTickableTransformConstraint::GetChannelsToKey() const
{
	static const TMap< ETransformConstraintType, EMovieSceneTransformChannel > ConstraintToChannels({
	{ETransformConstraintType::Translation, EMovieSceneTransformChannel::Translation},
	{ETransformConstraintType::Rotation, EMovieSceneTransformChannel::Rotation},
	{ETransformConstraintType::Scale, EMovieSceneTransformChannel::Scale},
	{ETransformConstraintType::Parent, EMovieSceneTransformChannel::AllTransform},
	{ETransformConstraintType::LookAt, EMovieSceneTransformChannel::Rotation}
		});

	const ETransformConstraintType ConstType = static_cast<ETransformConstraintType>(GetType());
	if (const EMovieSceneTransformChannel* Channel = ConstraintToChannels.Find(ConstType))
	{
		return *Channel;
	}

	return EMovieSceneTransformChannel::AllTransform;
}

UTickableConstraint* UTickableTransformConstraint::Duplicate(UObject* NewOuter) const
{
	UTickableTransformConstraint* Dup = DuplicateObject<UTickableTransformConstraint>(this, NewOuter);
	if (ChildTRSHandle)
	{
		UTransformableHandle* HandleCopy = ChildTRSHandle->Duplicate(Dup);
		Dup->ChildTRSHandle = HandleCopy;
	}
	if (ParentTRSHandle)
	{
		UTransformableHandle* HandleCopy = ParentTRSHandle->Duplicate(Dup);
		Dup->ParentTRSHandle = HandleCopy;
	}
	return Dup;
}

#if WITH_EDITOR

FString UTickableTransformConstraint::GetLabel() const
{
	if (::IsValid(ChildTRSHandle) && ::IsValid(ParentTRSHandle))
	{
		return FString::Printf(TEXT("%s|%s"), *ParentTRSHandle->GetLabel(), *ChildTRSHandle->GetLabel());
	}

	static const FString DummyLabel;
	return DummyLabel;
}

FString UTickableTransformConstraint::GetFullLabel() const
{
	if (::IsValid(ChildTRSHandle) && ::IsValid(ParentTRSHandle))
	{
		return FString::Printf(TEXT("%s|%s"), *ParentTRSHandle->GetFullLabel(), *ChildTRSHandle->GetFullLabel());
	}

	static const FString DummyLabel;
	return DummyLabel;
}

FString UTickableTransformConstraint::GetTypeLabel() const
{
	static const UEnum* TypeEnum = StaticEnum<ETransformConstraintType>();
	if (TypeEnum->IsValidEnumValue(GetType()))
	{
		return TypeEnum->GetNameStringByValue(GetType());
	}

	return Super::GetTypeLabel();
}

bool UTickableTransformConstraint::CanEditChange(const FProperty* InProperty) const
{
	if (InProperty)
	{
		const FName MemberPropertyName = InProperty->GetFName();
		if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UTickableTranslationConstraint, OffsetTranslation) ||
			MemberPropertyName == GET_MEMBER_NAME_CHECKED(UTickableRotationConstraint, OffsetRotation) ||
			MemberPropertyName == GET_MEMBER_NAME_CHECKED(UTickableScaleConstraint, OffsetScale) ||
			MemberPropertyName == GET_MEMBER_NAME_CHECKED(UTickableParentConstraint, OffsetTransform))
		{
			if (GetOuter() == GetTransientPackage())
			{
				return bMaintainOffset && !bUseCurrentOffset;
			}
		}
	}
	
	return Super::CanEditChange(InProperty);
}

void UTickableTransformConstraint::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		return;
	}
	
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UTickableTransformConstraint, bMaintainOffset))
	{
		Evaluate();
		return;
	}

	if (const FProperty* MemberProperty = PropertyChangedEvent.MemberProperty)
	{
		const FName MemberPropertyName = MemberProperty->GetFName();
		if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UTickableTranslationConstraint, OffsetTranslation) ||
			MemberPropertyName == GET_MEMBER_NAME_CHECKED(UTickableRotationConstraint, OffsetRotation) ||
			MemberPropertyName == GET_MEMBER_NAME_CHECKED(UTickableScaleConstraint, OffsetScale) ||
			MemberPropertyName == GET_MEMBER_NAME_CHECKED(UTickableParentConstraint, OffsetTransform))
		{
			OnConstraintChanged.Broadcast(this, PropertyChangedEvent);
			Evaluate();
		}
	}
}

void UTickableTransformConstraint::PostEditUndo()
{
	Super::PostEditUndo();
}

UTickableTransformConstraint::FOnConstraintChanged& UTickableTransformConstraint::GetOnConstraintChanged()
{
	return OnConstraintChanged;	
}

#endif

void UTickableTransformConstraint::UnregisterDelegates() const
{
	if (ChildTRSHandle)
	{
		ChildTRSHandle->HandleModified().RemoveAll(this);
	}
	if (ParentTRSHandle)
	{
		ParentTRSHandle->HandleModified().RemoveAll(this);
	}
}

void UTickableTransformConstraint::RegisterDelegates()
{
	UnregisterDelegates();

	if (ChildTRSHandle)
	{
		ChildTRSHandle->HandleModified().AddUObject(this, &UTickableTransformConstraint::OnHandleModified);
	}
	if (ParentTRSHandle)
	{
		ParentTRSHandle->HandleModified().AddUObject(this, &UTickableTransformConstraint::OnHandleModified);
	}	
}

void UTickableTransformConstraint::Setup()
{
	if (!ParentTRSHandle || !ChildTRSHandle || !ParentTRSHandle->IsValid() || !ChildTRSHandle->IsValid())
	{
		// handle error
		return;
	}
	
	ComputeOffset();

}

void UTickableTransformConstraint::SetupDependencies(const UWorld* InWorld)
{
	//we may not be the outer for old files so move it over
	if (ParentTRSHandle)
	{
		UTickableTransformConstraint* PTRS = ParentTRSHandle->GetTypedOuter<UTickableTransformConstraint>();
		if(PTRS != this)
		{
			ParentTRSHandle->Rename(nullptr, this, REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
		}
	}
	if (ChildTRSHandle)
	{
		UTickableTransformConstraint* CTRS = ChildTRSHandle->GetTypedOuter<UTickableTransformConstraint>();
		if (CTRS != this)
		{
			ChildTRSHandle->Rename(nullptr, this, REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
		}
	}

	if (!ensure(InWorld && InWorld->GetCurrentLevel()))
	{
		return;
	}

	FTickFunction* ParentTickFunction = GetParentHandleTickFunction();
	FTickFunction* ChildTickFunction = GetChildHandleTickFunction();

	if (UE::Private::ShouldForceParentDependency() && ParentTickFunction && (ChildTickFunction != ParentTickFunction))
	{
		// manage dependencies
		// force ConstraintTickFunction to tick after InParent does.
		// Note that this might not register anything if the parent can't tick (static meshes for instance)
		FConstraintTickFunction& ConstraintTick = GetTickFunction(InWorld);
		ConstraintTick.AddPrerequisite(ParentTRSHandle->GetPrerequisiteObject(), *ParentTickFunction);
	}

	// TODO also check for cycle dependencies here
	if (ChildTickFunction)
	{
		USkeletalMeshComponent* ChildOwner = Cast<USkeletalMeshComponent>(ChildTRSHandle->GetTarget().Get());
		if (ChildOwner == nullptr || UE::Private::ShouldForceChildDependency())
		{
			// force InChild to tick after ConstraintTickFunction does.
			// Note that this might not register anything if the child can't tick (static meshes for instance)
			// Also don't do if it's a skeletal mesh will cause cycle. (to be updated)
			FConstraintTickFunction& ConstraintTick = GetTickFunction(InWorld);
			ChildTickFunction->AddPrerequisite(this, ConstraintTick);
		}
	}
}

void UTickableTransformConstraint::EnsurePrimaryDependency(const UWorld* InWorld)
{
	if (!ensure(InWorld && InWorld->GetCurrentLevel()))
	{
		return;
	}
	
	const FTickFunction* ParentTickFunction = GetParentHandleTickFunction();
	const FTickFunction* ChildTickFunction = GetChildHandleTickFunction();
	if (ParentTickFunction && (ChildTickFunction != ParentTickFunction))
	{
		FConstraintTickFunction& ConstraintTick = GetTickFunction(InWorld);
		const TArray<FTickPrerequisite>& ParentPrerequisites = ConstraintTick.GetPrerequisites();
		if (ParentPrerequisites.IsEmpty())
		{
			// if the constraint has no prerex at this stage, this means that the parent tick function
			// is not registered or can't tick (static meshes for instance) so look for the first parent tick function if any.
			// In a context of adding several constraints, we want to make sure that the evaluation order is the right one
			FTickPrerequisite PrimaryPrerex = ParentTRSHandle->GetPrimaryPrerequisite(UE::Private::ShouldForceParentDependency());
			if (FTickFunction* PotentialFunction = PrimaryPrerex.Get())
			{
				TSet<const FTickFunction*> VisitedFunctions;
				if (!FConstraintCycleChecker::HasPrerequisiteDependencyWith(&ConstraintTick, PotentialFunction, VisitedFunctions))
				{
					if (FDependencyBuilder::LogDependencies())
					{
						UE_LOGF(LogTemp, Warning, "EnsurePrimaryDependency: '%ls' must tick before '%ls'",
							*PotentialFunction->DiagnosticMessage(),
							*ConstraintTick.DiagnosticMessage() );
					}
					
					UObject* Target = PrimaryPrerex.PrerequisiteObject.Get();
					ConstraintTick.AddPrerequisite(Target, *PotentialFunction);
				}
			}
		}
	}
}

void UTickableTransformConstraint::PostLoad()
{
	Super::PostLoad();
}

void UTickableTransformConstraint::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

}

uint32 UTickableTransformConstraint::GetTargetHash() const
{
	return (ChildTRSHandle && ChildTRSHandle->IsValid()) ? ChildTRSHandle->GetHash() : 0;
}

bool UTickableTransformConstraint::ReferencesObject(TWeakObjectPtr<UObject> InObject) const
{
	const TWeakObjectPtr<UObject> ChildTarget = (ChildTRSHandle && ChildTRSHandle->IsValid())? ChildTRSHandle->GetTarget() : nullptr;
	if (ChildTarget == InObject)
	{
		return true;
	}

	const TWeakObjectPtr<UObject> ParentTarget = (ParentTRSHandle && ParentTRSHandle->IsValid()) ? ParentTRSHandle->GetTarget() : nullptr;
	if (ParentTarget == InObject)
	{
		return true;	
	}
	
	return false;
}

bool UTickableTransformConstraint::HasBoundObjects() const
{
	if (ChildTRSHandle && ChildTRSHandle->HasBoundObjects())
	{
		return true;
	}
	if (ParentTRSHandle && ParentTRSHandle->HasBoundObjects())
	{
		return true;
	}
	return false;
}

void UTickableTransformConstraint::ResolveBoundObjects(FMovieSceneSequenceID LocalSequenceID, TSharedRef<UE::MovieScene::FSharedPlaybackState> SharedPlaybackState, UObject* SubObject)
{
	// update dependencies if the constraint becomes valid once resolved 
	FConstraintDependencyScope Scope(this);
	
	if (ChildTRSHandle && ChildTRSHandle->HasBoundObjects())
	{
		ChildTRSHandle->ResolveBoundObjects(LocalSequenceID, SharedPlaybackState, SubObject);
	}
	if (ParentTRSHandle && ParentTRSHandle->HasBoundObjects())
	{
		ParentTRSHandle->ResolveBoundObjects(LocalSequenceID, SharedPlaybackState, SubObject);
	}
}

void UTickableTransformConstraint::Evaluate(bool bTickHandlesAlso) const
{
	if (IsFullyActive())
	{
		if (bTickHandlesAlso)
		{
			if (TransformableHandleUtils::SkipTicking())
			{
				if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(ParentTRSHandle->GetTarget()))
				{
					TransformableHandleUtils::MarkComponentForEvaluation(SkeletalMeshComponent);
				}
			}
			else
			{
				ParentTRSHandle->TickTarget();
			}
		}
		Super::Evaluate();
	}
}

void UTickableTransformConstraint::SetActive(const bool bIsActive)
{
	Super::SetActive(bIsActive);
}

void UTickableTransformConstraint::SetChildGlobalTransform(const FTransform& InGlobal) const
{
	if(ChildTRSHandle && ChildTRSHandle->IsValid())
	{
		ChildTRSHandle->SetGlobalTransform(InGlobal);
	}
}

void UTickableTransformConstraint::SetChildLocalTransform(const FTransform& InLocal) const
{
	if(ChildTRSHandle && ChildTRSHandle->IsValid())
	{
		ChildTRSHandle->SetLocalTransform(InLocal);
	}
}

FTransform UTickableTransformConstraint::GetChildGlobalTransform() const
{
	return (ChildTRSHandle && ChildTRSHandle->IsValid()) ? ChildTRSHandle->GetGlobalTransform() : FTransform::Identity;
}

FTransform UTickableTransformConstraint::GetChildLocalTransform() const
{
	return (ChildTRSHandle && ChildTRSHandle->IsValid()) ? ChildTRSHandle->GetLocalTransform() : FTransform::Identity;
}

FTransform UTickableTransformConstraint::GetParentGlobalTransform() const
{
	return (ParentTRSHandle && ParentTRSHandle->IsValid()) ? ParentTRSHandle->GetGlobalTransform() : FTransform::Identity;
}

FTransform UTickableTransformConstraint::GetParentLocalTransform() const
{
	return (ParentTRSHandle && ParentTRSHandle->IsValid()) ? ParentTRSHandle->GetLocalTransform() : FTransform::Identity;
}

void UTickableTransformConstraint::OnHandleModified(UTransformableHandle* InHandle, EHandleEvent InNotification)
{
	if (!InHandle)
	{
		return;
	}

	const UObject* Target = InHandle->GetTarget().Get();
	UWorld* World = Target != nullptr ? Target->GetWorld() : nullptr;
	if (!World)
	{
		return;
	}

	const bool bIsThisChildHandle = InHandle == ChildTRSHandle;
	const bool bIsThisParentHandle = InHandle == ParentTRSHandle;

	// if the incoming handle has nothing to do with this constraint then exit
	if (!bIsThisChildHandle && !bIsThisParentHandle)
	{
		return;
	}

	// update dependencies now the component has been updated.
	if (InNotification == EHandleEvent::ComponentUpdated)
	{
		SetupDependencies(World);
		FDependencyBuilder::BuildDependencies(World, this);
		return;
	}

	auto MarkForEvaluation = [this, World]()
	{
		const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(World);
		Controller.MarkConstraintForEvaluation(this);
	};

	if (bIsThisChildHandle)
	{
		if (InNotification == EHandleEvent::UpperDependencyUpdated)
		{
			const UObject* ParentTarget = UE::Private::GetHandleTarget(ParentTRSHandle);
			if (ParentTarget && ParentTarget != Target)
			{
				return MarkForEvaluation();
			}
		}

		if (InNotification == EHandleEvent::GlobalTransformUpdated)
		{
			return MarkForEvaluation();
		}
	}

	if (bIsThisParentHandle)
	{
		if (InNotification == EHandleEvent::GlobalTransformUpdated || InNotification == EHandleEvent::UpperDependencyUpdated)
		{
			return MarkForEvaluation();
		}
	}
}

bool UTickableTransformConstraint::IsValid(const bool bDeepCheck) const
{
	const bool bAreHandlesValid =
		::IsValid(ChildTRSHandle) && ChildTRSHandle->IsValid(bDeepCheck) &&
		::IsValid(ParentTRSHandle) && ParentTRSHandle->IsValid(bDeepCheck); 
	
	return bDeepCheck ? bAreHandlesValid && bValid : bAreHandlesValid;
}

bool UTickableTransformConstraint::IsFullyActive() const
{
	return (Active && IsValid());
}

bool UTickableTransformConstraint::NeedsCompensation() const
{
	// NOTE: this can be extended to something more complex if needed  
	return true;
}

FTickFunction* UTickableTransformConstraint::GetChildHandleTickFunction() const 
{
	return GetHandleTickFunction(ChildTRSHandle); 
}

FTickFunction* UTickableTransformConstraint::GetParentHandleTickFunction() const 
{
	return GetHandleTickFunction(ParentTRSHandle); 
}

FTickFunction* UTickableTransformConstraint::GetHandleTickFunction(const TObjectPtr<UTransformableHandle>& InHandle) const 
{
	if (!::IsValid(InHandle) || !InHandle->IsValid())
	{
		return nullptr;
	}
		
	return InHandle->GetTickFunction();
}

void UTickableTransformConstraint::PreEvaluate() const
{
	UE::Private::PreEvaluateParent(ParentTRSHandle);
	UE::Private::PreEvaluateChild(ChildTRSHandle);
}

void UTickableTransformConstraint::PostEvaluate() const
{
	if (TransformableHandleUtils::SkipTicking())
	{
		return;	
	}
	
	if (ChildTRSHandle)
	{
		ChildTRSHandle->TickTarget();
	}
}

void UTickableTransformConstraint::InitConstraint(UWorld *InWorld)
{
	ULevel* Level = InWorld ? InWorld->GetCurrentLevel() : nullptr;
	if (!ensure(Level))
	{
		return;
	}

	
	FConstraintTickFunction& ConstraintTick = GetTickFunction(InWorld);

	if (ConstraintTick.ConstraintFunctions.IsEmpty())
	{
		ConstraintTick.RegisterFunction(GetFunction());
	}

	if (!ConstraintTick.IsTickFunctionRegistered())
	{
		ConstraintTick.RegisterTickFunction(Level);
	}
	
	ConstraintTick.Constraint = this;

	SetupDependencies(InWorld);
	RegisterDelegates();

	bValid = true;
}

void UTickableTransformConstraint::TeardownConstraint(UWorld* InWorld)
{
	if (!ensure(InWorld))
	{
		return;
	}

	const ULevel* Level = InWorld ? InWorld->GetCurrentLevel() : nullptr;


	TUniquePtr<FConstraintTickFunction>* Found = Level ? ConstraintTicks.Find(Level) : nullptr;
	if (Found && Found->IsValid())
	{
		TUniquePtr<FConstraintTickFunction>& ConstraintTick = *Found;
		
		ConstraintTick->UnRegisterTickFunction();
		ConstraintTick->SetTickFunctionEnable(false);

		if (FTickFunction* ChildTickFunction = GetChildHandleTickFunction())
		{
			ChildTickFunction->RemovePrerequisite(this, *ConstraintTick);
		}
		else
		{
			// If ChildTickFunction is null, meaning that the handle no longer references any component,
			// we must ensure that no component in this level still has it as a prerequisite.
			FConstraintTickFunction* ConstraintTickPtr = ConstraintTick.Get();

			static constexpr bool bEvenIfPendingKill = true;
			for (const TObjectPtr<AActor>& Actor: Level->Actors)
			{
				if (Actor)
				{
					for (UActorComponent* Component: Actor->GetComponents())
					{
						if (Component)
						{
							TArray<FTickPrerequisite>& Prerequisites = Component->PrimaryComponentTick.GetPrerequisites();
							for (int32 PrereqIndex = 0; PrereqIndex < Prerequisites.Num(); PrereqIndex++)
							{
								FTickPrerequisite& Prerequisite = Prerequisites[PrereqIndex];
								UObject* PrereqObject = Prerequisite.PrerequisiteObject.Get(bEvenIfPendingKill);
								if (PrereqObject == this && Prerequisite.Get() == ConstraintTickPtr)
								{
									Prerequisites.RemoveAtSwap(PrereqIndex--);
								}
							}
						}
					}
				}
			}
		}
	
		if (FTickFunction* ParentTickFunction = GetParentHandleTickFunction())
		{
			ConstraintTick->RemovePrerequisite(ParentTRSHandle->GetPrerequisiteObject(), *ParentTickFunction);	
		}

		ConstraintTicks.Remove(Level);
		
		if (ConstraintTicks.IsEmpty())
		{
			Active = false;
		}
	}

	// unregister delegates. should handles delegates be unregistered as well?
	UnregisterDelegates();
}

void UTickableTransformConstraint::AddedToWorld(UWorld* InWorld)
{
	if (InWorld)
	{
		// update dependencies once added to the sub-system 
		FDependencyBuilder::BuildDependencies(InWorld, this);
	}
}

/** 
 * UTickableTranslationConstraint
 **/

UTickableTranslationConstraint::UTickableTranslationConstraint()
{
	Type = ETransformConstraintType::Translation;
}

#if WITH_EDITOR

void UTickableTranslationConstraint::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		return;
	}

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UTickableTransformConstraint, bDynamicOffset))
	{
		if (bDynamicOffset)
		{
			Cache.CachedInputHash = CalculateInputHash();
			
			const FTransform ChildGlobalTransform = GetChildGlobalTransform();
			const FTransform ParentWorldTransform = GetParentGlobalTransform();
			OffsetTranslation = ChildGlobalTransform.GetLocation() - ParentWorldTransform.GetLocation();
			
			Evaluate();
		}
		return;
	}
}

#endif

void UTickableTranslationConstraint::ComputeOffset()
{
	const FTransform InitParentTransform = GetParentGlobalTransform();
	const FTransform InitChildTransform = GetChildGlobalTransform();
	
	OffsetTranslation = FVector::ZeroVector;
	if (bMaintainOffset || bDynamicOffset)
	{
		OffsetTranslation = InitChildTransform.GetLocation() - InitParentTransform.GetLocation();
	}	
}

FConstraintTickFunction::ConstraintFunction UTickableTranslationConstraint::GetFunction() const
{
	return [this]()
	{
		if (!IsFullyActive())
		{
			return;
		}
		
		const float ClampedWeight = FMath::Clamp<float>(Weight, 0.f, 1.f);
		if (ClampedWeight < KINDA_SMALL_NUMBER)
		{
			return;
		}

		PreEvaluate();
		
		const FVector ParentTranslation = GetParentGlobalTransform().GetLocation();
		FTransform Transform = GetChildGlobalTransform();
		const FVector ChildTranslation = Transform.GetLocation();
		
		FVector NewTranslation = (!bMaintainOffset) ? ParentTranslation : ParentTranslation + OffsetTranslation;
		if (ClampedWeight < 1.0f - KINDA_SMALL_NUMBER)
		{
			NewTranslation = FMath::Lerp<FVector>(ChildTranslation, NewTranslation, ClampedWeight);
		}

		AxisFilter.FilterVector(NewTranslation, ChildTranslation);

		Transform.SetLocation(NewTranslation);
			
		SetChildGlobalTransform(Transform);

		PostEvaluate();
	};
}

void UTickableTranslationConstraint::OnHandleModified(UTransformableHandle* InHandle, EHandleEvent InEvent)
{
	Super::OnHandleModified(InHandle, InEvent);
	
	if (!IsFullyActive() || !bDynamicOffset)
	{
		return;
	}
	
	if (!InHandle || InHandle != ChildTRSHandle)
	{
		return;
	}

	const bool bUpdateFromGlobal = (InEvent == EHandleEvent::GlobalTransformUpdated);
	const bool bUpdateTransform = InEvent == EHandleEvent::LocalTransformUpdated || bUpdateFromGlobal;
	if (!bUpdateTransform)
	{
		return;
	}

	const uint32 InputHash = CalculateInputHash();
	
	// update dynamic offset
	if (InputHash != Cache.CachedInputHash)
	{
		Cache.CachedInputHash = InputHash;

		if (bUpdateFromGlobal)
		{
			const FTransform ParentWorldTransform = GetParentGlobalTransform();
			const FTransform ChildGlobalTransform = GetChildGlobalTransform();
			OffsetTranslation = ChildGlobalTransform.GetLocation() - ParentWorldTransform.GetLocation();
		}
		else
		{
			const FTransform ChildLocalTransform = GetChildLocalTransform();
			OffsetTranslation = ChildLocalTransform.GetTranslation();
		}
	}
}

uint32 UTickableTranslationConstraint::CalculateInputHash() const
{
	uint32 Hash = 0;

	// local location hash
	const FTransform ChildLocalTransform = GetChildLocalTransform();
	Hash = HashCombine(Hash, GetTypeHash(ChildLocalTransform.GetTranslation() ));

	// global location hash
	const FTransform ChildGlobalTransform = GetChildGlobalTransform();
	Hash = HashCombine(Hash, GetTypeHash(ChildGlobalTransform.GetTranslation() ));
	
	return Hash;
}

/** 
 * UTickableRotationConstraint
 **/

UTickableRotationConstraint::UTickableRotationConstraint()
{
	Type = ETransformConstraintType::Rotation;
}

#if WITH_EDITOR

void UTickableRotationConstraint::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		return;
	}
	
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UTickableTransformConstraint, bDynamicOffset))
	{
		if (bDynamicOffset)
		{
			Cache.CachedInputHash = CalculateInputHash();
			
			const FTransform ChildGlobalTransform = GetChildGlobalTransform();
			const FTransform ParentWorldTransform = GetParentGlobalTransform();
			OffsetRotation = ParentWorldTransform.GetRotation().Inverse() * ChildGlobalTransform.GetRotation();
			
			Evaluate();
		}
		return;
	}
}

#endif

void UTickableRotationConstraint::ComputeOffset()
{
	const FTransform InitParentTransform = GetParentGlobalTransform();
	const FTransform InitChildTransform = GetChildGlobalTransform();
	
	OffsetRotation = FQuat::Identity;
	if (bMaintainOffset || bDynamicOffset)
	{
		OffsetRotation = InitParentTransform.GetRotation().Inverse() * InitChildTransform.GetRotation();
		OffsetRotation.Normalize(); 
	}
}

FConstraintTickFunction::ConstraintFunction UTickableRotationConstraint::GetFunction() const
{
	return [this]()
	{
		if (!IsFullyActive())
		{
			return;
		}
		
		const float ClampedWeight = FMath::Clamp<float>(Weight, 0.f, 1.f);
		if (ClampedWeight < KINDA_SMALL_NUMBER)
		{
			return;
		}

		PreEvaluate();
		
		const FQuat ParentRotation = GetParentGlobalTransform().GetRotation();
		FTransform Transform = GetChildGlobalTransform();
		const FQuat ChildRotation = Transform.GetRotation();

		FQuat NewRotation = (!bMaintainOffset) ? ParentRotation : ParentRotation * OffsetRotation;
		if (ClampedWeight < 1.0f - KINDA_SMALL_NUMBER)
		{
			NewRotation = FQuat::Slerp(ChildRotation, NewRotation, ClampedWeight);
		}

		AxisFilter.FilterQuat(NewRotation, ChildRotation);


		Transform.SetRotation(NewRotation);
		
		SetChildGlobalTransform(Transform);

		PostEvaluate();
	};
}

void UTickableRotationConstraint::OnHandleModified(UTransformableHandle* InHandle, EHandleEvent InEvent)
{
	Super::OnHandleModified(InHandle, InEvent);
	
	if (!IsFullyActive() || !bDynamicOffset)
	{
		return;
	}
	
	if (!InHandle || InHandle != ChildTRSHandle)
	{
		return;
	}

	const bool bUpdateFromGlobal = (InEvent == EHandleEvent::GlobalTransformUpdated);
	const bool bUpdateTransform = (InEvent == EHandleEvent::LocalTransformUpdated) || bUpdateFromGlobal;
	if (!bUpdateTransform)
	{
		return;
	}

	const uint32 InputHash = CalculateInputHash();
	
	// update dynamic offset
	if (InputHash != Cache.CachedInputHash)
	{
		Cache.CachedInputHash = InputHash;

		if (bUpdateFromGlobal)
		{
			const FTransform ParentWorldTransform = GetParentGlobalTransform();
			const FTransform ChildGlobalTransform = GetChildGlobalTransform();
			OffsetRotation = ParentWorldTransform.GetRotation().Inverse() * ChildGlobalTransform.GetRotation();
		}
		else
		{
			OffsetRotation = GetChildLocalTransform().GetRotation();
		}
	}
}

uint32 UTickableRotationConstraint::CalculateInputHash() const
{
	uint32 Hash = 0;

	// local rotation hash
	const FTransform ChildLocalTransform = GetChildLocalTransform();
	Hash = HashCombine(Hash, GetTypeHash(ChildLocalTransform.GetRotation().Euler() ));

	// global rotation hash
	const FTransform ChildGlobalTransform = GetChildGlobalTransform();
	Hash = HashCombine(Hash, GetTypeHash(ChildGlobalTransform.GetRotation().Euler() ));
	
	return Hash;
}

/** 
 * UTickableScaleConstraint
 **/

UTickableScaleConstraint::UTickableScaleConstraint()
{
	Type = ETransformConstraintType::Scale;
}

#if WITH_EDITOR

void UTickableScaleConstraint::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		return;
	}
	
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UTickableTransformConstraint, bDynamicOffset))
	{
		if (bDynamicOffset)
		{
			Cache.CachedInputHash = CalculateInputHash();
			
			const FVector ParentScale = GetParentGlobalTransform().GetScale3D();
			OffsetScale = GetChildGlobalTransform().GetScale3D();
			OffsetScale[0] = FMath::Abs(ParentScale[0]) > KINDA_SMALL_NUMBER ? OffsetScale[0] / ParentScale[0] : 0.f;
			OffsetScale[1] = FMath::Abs(ParentScale[1]) > KINDA_SMALL_NUMBER ? OffsetScale[1] / ParentScale[1] : 0.f;
			OffsetScale[2] = FMath::Abs(ParentScale[2]) > KINDA_SMALL_NUMBER ? OffsetScale[2] / ParentScale[2] : 0.f;
			
			Evaluate();
		}
		return;
	}
}

#endif

void UTickableScaleConstraint::ComputeOffset()
{
	const FTransform InitParentTransform = GetParentGlobalTransform();
	const FTransform InitChildTransform = GetChildGlobalTransform();
	
	OffsetScale = FVector::OneVector;
	if (bMaintainOffset)
	{
		const FVector InitParentScale = InitParentTransform.GetScale3D();
		OffsetScale = InitChildTransform.GetScale3D();
		OffsetScale[0] = FMath::Abs(InitParentScale[0]) > KINDA_SMALL_NUMBER ? OffsetScale[0] / InitParentScale[0] : 0.f;
		OffsetScale[1] = FMath::Abs(InitParentScale[1]) > KINDA_SMALL_NUMBER ? OffsetScale[1] / InitParentScale[1] : 0.f;
		OffsetScale[2] = FMath::Abs(InitParentScale[2]) > KINDA_SMALL_NUMBER ? OffsetScale[2] / InitParentScale[2] : 0.f;
	}
}

FConstraintTickFunction::ConstraintFunction UTickableScaleConstraint::GetFunction() const
{
	return [this]()
	{
		if (!IsFullyActive())
		{
			return;
		}
		
		const float ClampedWeight = FMath::Clamp<float>(Weight, 0.f, 1.f);
		if (ClampedWeight < KINDA_SMALL_NUMBER)
		{
			return;
		}

		PreEvaluate();
		
		const FVector ParentScale = GetParentGlobalTransform().GetScale3D();
		FTransform Transform = GetChildGlobalTransform();
		const FVector ChildScale = Transform.GetScale3D();
		
		FVector NewScale = (!bMaintainOffset) ? ParentScale : ParentScale * OffsetScale;
		if (ClampedWeight < 1.0f - KINDA_SMALL_NUMBER)
		{
			NewScale = FMath::Lerp<FVector>(ChildScale, NewScale, ClampedWeight);
		}

		AxisFilter.FilterVector(NewScale, ChildScale);

		Transform.SetScale3D(NewScale);
		
		SetChildGlobalTransform(Transform);

		PostEvaluate();
	};
}

void UTickableScaleConstraint::OnHandleModified(UTransformableHandle* InHandle, EHandleEvent InEvent)
{
	Super::OnHandleModified(InHandle, InEvent);
	
	if (!IsFullyActive() || !bDynamicOffset)
	{
		return;
	}
	
	if (!InHandle || InHandle != ChildTRSHandle)
	{
		return;
	}

	const bool bUpdateFromGlobal = (InEvent == EHandleEvent::GlobalTransformUpdated);
	const bool bUpdateTransform = InEvent == EHandleEvent::LocalTransformUpdated || bUpdateFromGlobal;
	if (!bUpdateTransform)
	{
		return;
	}

	const uint32 InputHash = CalculateInputHash();
	
	// update dynamic offset
	if (InputHash != Cache.CachedInputHash)
	{
		Cache.CachedInputHash = InputHash;

		if (bUpdateFromGlobal)
		{
			const FVector ParentScale = GetParentGlobalTransform().GetScale3D();
			OffsetScale = GetChildGlobalTransform().GetScale3D();
			OffsetScale[0] = FMath::Abs(ParentScale[0]) > KINDA_SMALL_NUMBER ? OffsetScale[0] / ParentScale[0] : 0.f;
			OffsetScale[1] = FMath::Abs(ParentScale[1]) > KINDA_SMALL_NUMBER ? OffsetScale[1] / ParentScale[1] : 0.f;
			OffsetScale[2] = FMath::Abs(ParentScale[2]) > KINDA_SMALL_NUMBER ? OffsetScale[2] / ParentScale[2] : 0.f;
		}
		else
		{
			const FTransform ChildLocalTransform = GetChildLocalTransform();
			OffsetScale = ChildLocalTransform.GetScale3D();
		}
	}
}

uint32 UTickableScaleConstraint::CalculateInputHash() const
{
	uint32 Hash = 0;

	// local scale hash
	const FTransform ChildLocalTransform = GetChildLocalTransform();
	Hash = HashCombine(Hash, GetTypeHash(ChildLocalTransform.GetScale3D() ));

	// global scale hash
	const FTransform ChildGlobalTransform = GetChildGlobalTransform();
	Hash = HashCombine(Hash, GetTypeHash(ChildGlobalTransform.GetScale3D() ));
	
	return Hash;
}

/** 
 * UTickableParentConstraint
 **/

UTickableParentConstraint::UTickableParentConstraint()
{
	Type = ETransformConstraintType::Parent;
}

void UTickableParentConstraint::ComputeOffset()
{
	const FTransform InitParentTransform = GetParentGlobalTransform();
	FTransform InitChildTransform = GetChildGlobalTransform();
	
	OffsetTransform = FTransform::Identity;
	if (bMaintainOffset || bDynamicOffset)
	{
		if (!bScaling)
		{
			InitChildTransform.RemoveScaling();
		}
		OffsetTransform = InitChildTransform.GetRelativeTransform(InitParentTransform); 
	}
}

uint32 UTickableParentConstraint::CalculateInputHash() const
{
	uint32 Hash = 0;
	
	const FTransform ChildLocalTransform = GetChildLocalTransform();
	Hash = HashCombine(Hash, GetTypeHash(ChildLocalTransform.GetTranslation()));
	Hash = HashCombine(Hash, GetTypeHash(ChildLocalTransform.GetRotation().Euler() ));
	Hash = HashCombine(Hash, GetTypeHash(ChildLocalTransform.GetScale3D()));
	
	const FTransform ChildGlobalTransform = GetChildGlobalTransform();
	Hash = HashCombine(Hash, GetTypeHash(ChildGlobalTransform.GetTranslation()));
	Hash = HashCombine(Hash, GetTypeHash(ChildGlobalTransform.GetRotation().Euler() ));
	Hash = HashCombine(Hash, GetTypeHash(ChildGlobalTransform.GetScale3D()));
	
	return Hash;
}

FConstraintTickFunction::ConstraintFunction UTickableParentConstraint::GetFunction() const
{
	return [this]()
	{
		if (!IsFullyActive())
		{
			return;
		}
		
		const float ClampedWeight = FMath::Clamp<float>(Weight, 0.f, 1.f);
		if (ClampedWeight < KINDA_SMALL_NUMBER)
		{
			return;
		}

		auto LerpAndFilterTransform = [ClampedWeight](
			const FTransform& InTransform, FTransform& OutTransform, const FTransformFilter& InFilter)
		{
			const FVector Location = InTransform.GetLocation();
			const FQuat Rotation = InTransform.GetRotation();
			const FVector Scale = InTransform.GetScale3D();

			FVector NewLocation = OutTransform.GetLocation();
			FQuat NewRotation = OutTransform.GetRotation();
			FVector NewScale = OutTransform.GetScale3D();
			
			if (ClampedWeight < 1.0f - KINDA_SMALL_NUMBER)
			{
				NewLocation = FMath::Lerp<FVector>(Location, NewLocation, ClampedWeight);
				NewRotation = FQuat::Slerp(Rotation, NewRotation, ClampedWeight);
				NewScale = FMath::Lerp<FVector>(Scale, NewScale, ClampedWeight);
			}

			InFilter.TranslationFilter.FilterVector(NewLocation, Location);
			InFilter.RotationFilter.FilterQuat(NewRotation, Rotation);
			InFilter.ScaleFilter.FilterVector(NewScale, Scale);

			OutTransform.SetLocation(NewLocation);
			OutTransform.SetRotation(NewRotation);
			OutTransform.SetScale3D(NewScale);
		};

		PreEvaluate();
		
		const FTransform ParentTransform = GetParentGlobalTransform();
		
		FTransform TargetTransform = (!bMaintainOffset) ? ParentTransform : OffsetTransform * ParentTransform;
		//apply weight if needed
		const FTransform ChildGlobalTransform = GetChildGlobalTransform();
		LerpAndFilterTransform(ChildGlobalTransform, TargetTransform, TransformFilter);

		//remove scale?
		if (!bScaling)
		{
			TargetTransform.SetScale3D(ChildGlobalTransform.GetScale3D());
		}

		SetChildGlobalTransform(TargetTransform);

		PostEvaluate();
	};
}

void UTickableParentConstraint::OnHandleModified(UTransformableHandle* InHandle, EHandleEvent InEvent)
{
	Super::OnHandleModified(InHandle, InEvent);
	
	if (!IsFullyActive() || !bDynamicOffset)
	{
		return;
	}
	
	if (!InHandle || InHandle != ChildTRSHandle)
	{
		return;
	}

	const bool bUpdateFromGlobal = (InEvent == EHandleEvent::GlobalTransformUpdated);
	const bool bUpdateTransform = (InEvent == EHandleEvent::LocalTransformUpdated) || bUpdateFromGlobal;
	if (!bUpdateTransform)
	{
		return;
	}
	
	const uint32 InputHash = CalculateInputHash();

	// update dynamic offset
	if (InputHash != Cache.CachedInputHash)
	{
		Cache.CachedInputHash = InputHash;

		if (bUpdateFromGlobal)
		{
			const FTransform ParentWorldTransform = GetParentGlobalTransform();
			FTransform ChildGlobalTransform = GetChildGlobalTransform();
			if (!bScaling)
			{
				ChildGlobalTransform.RemoveScaling();
			}
			OffsetTransform = ChildGlobalTransform.GetRelativeTransform(ParentWorldTransform);
		}
		else
		{
			OffsetTransform = GetChildLocalTransform();
		}
	}
}

#if WITH_EDITOR

void UTickableParentConstraint::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		return;
	}
	
	auto UpdateOffset = [&]()
	{
		FTransform ChildGlobalTransform = GetChildGlobalTransform();
		if (!bScaling)
		{
			ChildGlobalTransform.RemoveScaling();
		}
		const FTransform ParentWorldTransform = GetParentGlobalTransform();
		OffsetTransform = ChildGlobalTransform.GetRelativeTransform(ParentWorldTransform);
	};
	
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UTickableTransformConstraint, bDynamicOffset))
	{
		if(bDynamicOffset)
		{
			Cache.CachedInputHash = CalculateInputHash();
			UpdateOffset();			
			Evaluate();
		}
		return;
	}

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UTickableParentConstraint, bScaling))
	{
		// notify scale change. note that this is currently the only property change we monitor but this broadcast
		// call could be made higher in the hierarchy to monitor other changes.
		OnConstraintChanged.Broadcast(this, PropertyChangedEvent);
		
		if (bMaintainOffset || bDynamicOffset)
		{
			Cache.CachedInputHash = CalculateInputHash();
			UpdateOffset();
			Evaluate();
			return;
		}
	}
}

#endif

/** 
 * UTickableLookAtConstraint
 **/

UTickableLookAtConstraint::UTickableLookAtConstraint()
{
	bMaintainOffset = false;
	bDynamicOffset = false;
	Type = ETransformConstraintType::LookAt;
}

void UTickableLookAtConstraint::ComputeOffset()
{
	bMaintainOffset = false;
	bDynamicOffset = false;

	const FVector InitParentLocation = GetParentGlobalTransform().GetLocation();
	const FTransform InitChildTransform = GetChildGlobalTransform();
	const FVector InitLookAtDir = (InitParentLocation - InitChildTransform.GetLocation()).GetSafeNormal();

	if (!InitLookAtDir.IsNearlyZero())
	{
		Axis = InitChildTransform.InverseTransformVectorNoScale(InitLookAtDir).GetSafeNormal();
	}
}

FConstraintTickFunction::ConstraintFunction UTickableLookAtConstraint::GetFunction() const
{
	return [this]()
	{
		if (!IsFullyActive())
		{
			return;
		}

		PreEvaluate();
		
		const FTransform ParentTransform = GetParentGlobalTransform();
		const FTransform ChildTransform = GetChildGlobalTransform();
		
		const FVector LookAtDir = (ParentTransform.GetLocation() - ChildTransform.GetLocation()).GetSafeNormal();

		if (!LookAtDir.IsNearlyZero() && !Axis.IsNearlyZero())
		{
			const FVector AxisToOrient = ChildTransform.TransformVectorNoScale(Axis).GetSafeNormal();
		
			FQuat Rotation = FindQuatBetweenNormals(AxisToOrient, LookAtDir);
			const bool bNeedsToBeRotated = !Rotation.IsIdentity();
			if (bNeedsToBeRotated)
			{
				Rotation = Rotation * ChildTransform.GetRotation();

				FTransform Transform = ChildTransform;
				Transform.SetRotation(Rotation.GetNormalized());
				SetChildGlobalTransform(Transform);

				PostEvaluate();
			}
		}
	};
}

bool UTickableLookAtConstraint::NeedsCompensation() const
{
	return false;
}

FQuat UTickableLookAtConstraint::FindQuatBetweenNormals(const FVector& A, const FVector& B)
{
	const FQuat::FReal Dot = FVector::DotProduct(A, B);
	FQuat::FReal W = 1 + Dot;
	FQuat Result;

	if (W < SMALL_NUMBER)
	{
		// A and B point in opposite directions
		W = 2 - W;
		Result = FQuat( -A.Y * B.Z + A.Z * B.Y, -A.Z * B.X + A.X * B.Z, -A.X * B.Y + A.Y * B.X, W).GetNormalized();

		const FVector Normal = FMath::Abs(A.X) > FMath::Abs(A.Y) ? FVector::YAxisVector : FVector::XAxisVector;
		const FVector BiNormal = FVector::CrossProduct(A, Normal);
		const FVector TauNormal = FVector::CrossProduct(A, BiNormal);
		Result = Result * FQuat(TauNormal, PI);
	}
	else
	{
		//Axis = FVector::CrossProduct(A, B);
		Result = FQuat( A.Y * B.Z - A.Z * B.Y, A.Z * B.X - A.X * B.Z, A.X * B.Y - A.Y * B.X, W);
	}

	Result.Normalize();
	return Result;
}

#if WITH_EDITOR

void UTickableLookAtConstraint::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		return;
	}

	const FName PropertyName = PropertyChangedEvent.GetMemberPropertyName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UTickableLookAtConstraint, Axis))
	{
		Evaluate();
		return;
	}
}

#endif

/** 
 * DEPRECATED FTransformConstraintUtils
 **/

UTransformableComponentHandle* FTransformConstraintUtils::CreateHandleForSceneComponent(
	USceneComponent* InSceneComponent,
	const FName& InSocketName)
{
	return UE::TransformConstraintUtil::CreateHandleForSceneComponent(InSceneComponent, InSocketName);
}

void FTransformConstraintUtils::GetParentConstraints(
	UWorld* InWorld,
	const AActor* InChild,
	TArray< TWeakObjectPtr<UTickableConstraint> >& OutConstraints)
{
	return UE::TransformConstraintUtil::GetParentConstraints(InWorld, InChild, OutConstraints);
}

UTickableTransformConstraint* FTransformConstraintUtils::CreateFromType(
	UWorld* InWorld,
	const ETransformConstraintType InType,
	const bool bUseDefault)
{
	return UE::TransformConstraintUtil::CreateFromType(InWorld, InType, bUseDefault);
}

UTickableTransformConstraint* FTransformConstraintUtils::CreateAndAddFromObjects(
	UWorld* InWorld,
	UObject* InParent, const FName& InParentSocketName,
	UObject* InChild, const FName& InChildSocketName,
	const ETransformConstraintType InType,
	const bool bMaintainOffset,
	const bool bUseDefault)
{
	return UE::TransformConstraintUtil::CreateAndAddFromObjects(
		InWorld, InParent, InParentSocketName, InChild, InChildSocketName, InType, bMaintainOffset, bUseDefault);
}

bool FTransformConstraintUtils::AddConstraint(
	UWorld* InWorld,
	UTransformableHandle* InParentHandle,
	UTransformableHandle* InChildHandle,
	UTickableTransformConstraint* InNewConstraint,
	const bool bMaintainOffset,
	const bool bUseDefault)
{
	return UE::TransformConstraintUtil::AddConstraint(InWorld, InParentHandle, InChildHandle, InNewConstraint, bMaintainOffset, bUseDefault);
}

void FTransformConstraintUtils::UpdateTransformBasedOnConstraint(FTransform& CurrentTransform, USceneComponent* SceneComponent)
{
	return UE::TransformConstraintUtil::UpdateTransformBasedOnConstraint(CurrentTransform, SceneComponent);
}

FTransform FTransformConstraintUtils::ComputeRelativeTransform(
	const FTransform& InChildLocal,
	const FTransform& InChildWorld,
	const FTransform& InSpaceWorld,
	const UTickableTransformConstraint* InConstraint)
{
	return UE::TransformConstraintUtil::ComputeRelativeTransform(InChildLocal, InChildWorld, InSpaceWorld, InConstraint);
}

TOptional<FTransform> FTransformConstraintUtils::GetRelativeTransform(UWorld* InWorld, const uint32 InHandleHash)
{
	return UE::TransformConstraintUtil::GetRelativeTransform(InWorld, InHandleHash);
}

TOptional<FTransform> FTransformConstraintUtils::GetConstraintsRelativeTransform(
	const TArray< TWeakObjectPtr<UTickableConstraint> >& InConstraints,
	const FTransform& InChildLocal, const FTransform& InChildWorld)
{
	return UE::TransformConstraintUtil::GetConstraintsRelativeTransform(InConstraints, InChildLocal, InChildWorld);
}

int32 FTransformConstraintUtils::GetLastActiveConstraintIndex(const TArray< TWeakObjectPtr<UTickableConstraint> >& InConstraints)
{
	return UE::TransformConstraintUtil::GetLastActiveConstraintIndex(InConstraints);
}

void FTransformConstraintUtils::GetChildrenConstraints(
	UWorld* World,
	const UTickableTransformConstraint* InConstraint,
	TArray< TWeakObjectPtr<UTickableConstraint> >& OutConstraints,
	const bool bIncludeTarget)
{
	return UE::TransformConstraintUtil::GetChildrenConstraints(World, InConstraint, OutConstraints, bIncludeTarget);
}
