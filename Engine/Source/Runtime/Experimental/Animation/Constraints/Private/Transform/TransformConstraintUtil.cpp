// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transform/TransformConstraintUtil.h"

#include "ConstraintsManager.inl"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "MovieSceneSection.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Transform/TransformableHandle.h"
#include "Transform/TransformableRegistry.h"
#include "Transform/TransformConstraint.h"
#include "Transform/DependencyBuilder.h"
#include "UObject/Object.h"
#include "UObject/Package.h"
#include "Widgets/Notifications/SNotificationList.h"

namespace UE::TransformConstraintUtil
{

namespace Private
{

uint32 GetHandleHash(const UObject* InObject, const FName& InAttachmentName = NAME_None)
{
	if (InObject)
	{
		const FTransformableRegistry& Registry = FTransformableRegistry::Get();
		if (const FTransformableRegistry::GetHashFuncT HashFunction = Registry.GetHashFunction(InObject->GetClass()))
		{
			return HashFunction(InObject, InAttachmentName);
		}
	}

	return 0;
}

UTransformableHandle* GetHandle(UObject* InObject, const FName& InSocketName)
{
	// look for customized transform handle
	const FTransformableRegistry& Registry = FTransformableRegistry::Get();
	if (const FTransformableRegistry::CreateHandleFuncT CreateFunction = Registry.GetCreateFunction(InObject->GetClass()))
	{
		return CreateFunction(InObject, InSocketName);
	}

	return nullptr;
}

UObject* GetTarget(const TObjectPtr<UTransformableHandle>& InHandle)
{
	return IsValid(InHandle) ? InHandle->GetTarget().Get() : nullptr;
}

void NotifyError(const FText& InNotification)
{
	// output log notification
	UE_LOGF(LogAnimation, Error, "%ls", *InNotification.ToString());
	
#if WITH_EDITOR
	// editor notification
	FNotificationInfo Info(InNotification);
	Info.Image = FAppStyle::GetBrush(TEXT("MessageLog.Error"));
	Info.ExpireDuration = 5.0f;

	FSlateNotificationManager::Get().AddNotification(Info);
#endif
}
	
}

const TArray< TWeakObjectPtr<UTickableConstraint> >& FConstraintsInteractionCache::Get(const UObject* InObject, const FName& InAttachmentName)
{
	const uint32 HandleHash = Private::GetHandleHash(InObject, InAttachmentName);
	if (HandleHash == 0)
	{
		static const TArray< TWeakObjectPtr<UTickableConstraint> > Empty;
		return Empty;
	}

	return Get(HandleHash, InObject->GetWorld());
}

bool FConstraintsInteractionCache::HasAnyActiveConstraint(const UObject* InObject, const FName& InAttachmentName)
{
	const uint32 HandleHash = Private::GetHandleHash(InObject, InAttachmentName);
	if (HandleHash == 0)
	{
		return false;
	}
	
	return HasAnyActiveConstraint(HandleHash, InObject->GetWorld());
}
	
TOptional<FTransform> FConstraintsInteractionCache::GetParentTransform(const UObject* InObject, const FName& InAttachmentName)
{
	const uint32 HandleHash = Private::GetHandleHash(InObject, InAttachmentName);
	if (HandleHash == 0)
	{
		static const TOptional<FTransform> Invalid;
		return Invalid;
	}

	return GetParentTransform(HandleHash, InObject->GetWorld());
}

const TArray< TWeakObjectPtr<UTickableConstraint> >& FConstraintsInteractionCache::Get(const uint32 InHandleHash, UWorld* InWorld)
{
	static const TArray< TWeakObjectPtr<UTickableConstraint> > Empty;
	if (!InWorld || InHandleHash == 0)
	{
		return Empty;
	}

	if (TArray< TWeakObjectPtr<UTickableConstraint> >* FoundConstraints = PerHandleConstraints.Find(InHandleHash))
	{
		return *FoundConstraints;
	}
	
	static constexpr bool bSorted = true;
	const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(InWorld);
	const TArray< TWeakObjectPtr<UTickableConstraint> > Constraints = Controller.GetParentConstraints(InHandleHash, bSorted);

	TArray< TWeakObjectPtr<UTickableConstraint> >& TransformConstraints = PerHandleConstraints.Add(InHandleHash);
	TransformConstraints = Constraints.FilterByPredicate([](const TWeakObjectPtr<UTickableConstraint>& WeakConstraint)
	{
		return Cast<UTickableTransformConstraint>(WeakConstraint.Get()) != nullptr;
	});
	return TransformConstraints;
}

bool FConstraintsInteractionCache::HasAnyActiveConstraint(const uint32 InHandleHash, UWorld* InWorld)
{
	const TArray< TWeakObjectPtr<UTickableConstraint> >& TransformConstraints = Get(InHandleHash, InWorld);
	return GetLastActiveConstraintIndex(TransformConstraints) != INDEX_NONE;
}

TOptional<FTransform> FConstraintsInteractionCache::GetParentTransform(const uint32 InHandleHash, UWorld* InWorld)
{
	const TArray< TWeakObjectPtr<UTickableConstraint> >& TransformConstraints = Get(InHandleHash, InWorld);
	const int32 ConstraintsIndex = GetLastActiveConstraintIndex(TransformConstraints);
	if (ConstraintsIndex != INDEX_NONE)
	{
		UTickableTransformConstraint* Constraint = Cast<UTickableTransformConstraint>(TransformConstraints[ConstraintsIndex].Get());
		if (Constraint && IsValid(Constraint->ParentTRSHandle))
		{
			return Constraint->ParentTRSHandle->GetGlobalTransform();	
		}
	}

	static const TOptional<FTransform> Invalid;
	return Invalid;
}

const TArray< TWeakObjectPtr<UTickableConstraint> >& FConstraintsInteractionCache::Get(UObject* InTarget, UWorld* InWorld)
{
	static const TArray< TWeakObjectPtr<UTickableConstraint> > Empty;
	if (!InTarget)
	{
		return Empty;
	}
	
	UWorld* World = InWorld ? InWorld : InTarget->GetWorld();
	if (!World)
	{
		return Empty;
	}

	if (TArray< TWeakObjectPtr<UTickableConstraint> >* FoundConstraints = PerTargetConstraints.Find(InTarget))
	{
		return *FoundConstraints;
	}

	TArray< TWeakObjectPtr<UTickableConstraint> >& TransformConstraints = PerTargetConstraints.Add(InTarget);
	
	static constexpr bool bSorted = true;
	const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(World);
	auto SameTargetPredicate = [InTarget](const TWeakObjectPtr<UTickableConstraint>& WeakConstraint)
	{
		if (const UTickableTransformConstraint* TransformConstraint = Cast<UTickableTransformConstraint>(WeakConstraint.Get()))
		{
			const UObject* Target = Private::GetTarget(TransformConstraint->ChildTRSHandle);
			return Target && Target == InTarget;
		}
		return false;
	};
	TransformConstraints = Controller.GetConstraintsByPredicate(SameTargetPredicate, bSorted);

	return TransformConstraints;
}
	
bool FConstraintsInteractionCache::HasAnyActiveConstraint(UObject* InTarget, UWorld* InWorld)
{
	const TArray< TWeakObjectPtr<UTickableConstraint> >& TransformConstraints = Get(InTarget, InWorld);
	return TransformConstraints.ContainsByPredicate([](const TWeakObjectPtr<UTickableConstraint>& InConstraint)
	{
	   if (const UTickableTransformConstraint* TransformConstraint = Cast<UTickableTransformConstraint>(InConstraint.Get()))
	   {
		   return TransformConstraint->Active;
	   }
	   return false;
	});
}

bool FConstraintsInteractionCache::HasAnyDependency(UObject* InChild, UObject* InParent, UWorld* InWorld)
{
	TSet< UObject* > Visited;
	return HasAnyDependencyInternal(InChild, InParent, InWorld, Visited);
}

bool FConstraintsInteractionCache::HasAnyDependencyInternal(UObject* InChild, UObject* InParent, UWorld* InWorld, TSet< UObject* >& Visited)
{
	if (!InChild || !InParent || !InWorld || InChild == InParent)
	{
		return false;
	}
	
	if (!HasAnyActiveConstraint(InChild, InWorld))
	{
		return false;
	}

	const TArray< TWeakObjectPtr<UTickableConstraint> >& Constraints = Get(InChild, InWorld);
	TSet< UObject* > Parents;
	Parents.Reserve(Constraints.Num());
	const bool bIsInParentAnActualParent = Constraints.ContainsByPredicate([InParent, &Parents](const TWeakObjectPtr<UTickableConstraint>& InConstraint)
	{
	   if (const UTickableTransformConstraint* TransformConstraint = Cast<UTickableTransformConstraint>(InConstraint.Get()))
	   {
		   if (UObject* Parent = Private::GetTarget(TransformConstraint->ParentTRSHandle))
		   {
			   Parents.Add(Parent);
			   if (Parent == InParent && InConstraint->Active)
			   {
				   return true;
			   }
		   }
	   }
	   return false;
	});

	if (bIsInParentAnActualParent)
	{
		return true;
	}

	if (Visited.Contains(InChild))
	{
		return false;
	}
	Visited.Add(InChild);
	
	for (UObject* Parent: Parents)
	{
		if (HasAnyDependencyInternal(Parent, InParent, InWorld, Visited))
		{
			return true;
		}
	}
		
	return false;
}

FConstraintsInteractionCache::~FConstraintsInteractionCache()
{
	Reset();
	UnregisterNotifications();
}
	
void FConstraintsInteractionCache::Reset()
{
	PerHandleConstraints.Reset();
	PerTargetConstraints.Reset();
}

void FConstraintsInteractionCache::RegisterNotifications()
{
	if (!ConstraintsNotificationHandle.IsValid())
	{
		ConstraintsNotificationHandle =
		   FConstraintsManagerController::GetNotifyDelegate().AddLambda([this](EConstraintsManagerNotifyType InNotifyType, UObject* InObject)
	   {
		   if (InNotifyType == EConstraintsManagerNotifyType::GraphUpdated)
		   {
			   Reset();
		   }
	   });
	}
}

void FConstraintsInteractionCache::UnregisterNotifications()
{
	if (ConstraintsNotificationHandle.IsValid())
	{
		FConstraintsManagerController::GetNotifyDelegate().Remove(ConstraintsNotificationHandle);
		ConstraintsNotificationHandle.Reset();
	}
}

UTransformableComponentHandle* CreateHandleForSceneComponent(USceneComponent* InSceneComponent, const FName& InSocketName)
{
	UTransformableComponentHandle* ComponentHandle = nullptr;
	if (InSceneComponent)
	{
		ComponentHandle = NewObject<UTransformableComponentHandle>(GetTransientPackage(), NAME_None, RF_Transactional);
		ComponentHandle->Component = InSceneComponent;
		ComponentHandle->SocketName = InSocketName;
		InSceneComponent->SetMobility(EComponentMobility::Movable);
		ComponentHandle->RegisterDelegates();
	}
	return ComponentHandle;
}

void GetParentConstraints( UWorld* InWorld, const AActor* InChild, TArray< TWeakObjectPtr<UTickableConstraint> >& OutConstraints)
{
	if (!InWorld || !InChild)
	{
		return;
	}

	const uint32 ChildHash = Private::GetHandleHash(InChild);
	if (ChildHash == 0)
	{
		return;
	}
	
	static constexpr bool bSorted = true;
	const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(InWorld);
	OutConstraints.Append(Controller.GetParentConstraints(ChildHash, bSorted));
}

UTickableTransformConstraint* CreateFromType(UWorld* InWorld, const ETransformConstraintType InType, const bool bUseDefault)
{
	if (!InWorld)
	{
		const FText Notification =
			NSLOCTEXT("TransformConstraintUtil", "CreateFromType_NoWorld", "Cannot create a new constraint in a null world.");
		Private::NotifyError(Notification);
		return nullptr;
	}
	
	const UEnum* ETransformConstraintTypeEnum = StaticEnum<ETransformConstraintType>();
	if (!ETransformConstraintTypeEnum->IsValidEnumValue(static_cast<int64>(InType)))
	{
		const FText Notification = FText::Format(
			NSLOCTEXT("TransformConstraintUtil", "CreateFromType_UnknownType",  "Cannot create a new constraint with unknown type '{0}'."),
			FText::AsNumber(static_cast<int32>(InType)));
		Private::NotifyError(Notification);
		return nullptr;
	}


	// unique name (we may want to use another approach here to manage uniqueness)
	const FString ConstraintTypeStr = ETransformConstraintTypeEnum->GetNameStringByValue((uint8)InType);
	const FName BaseName(*FString::Printf(TEXT("%sConstraint"), *ConstraintTypeStr));

	const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(InWorld);
	UTickableTransformConstraint* Constraint = nullptr;

	switch (InType)
	{
	case ETransformConstraintType::Translation:
		Constraint = Controller.AllocateConstraintT<UTickableTranslationConstraint>(BaseName, bUseDefault);
		break;
	case ETransformConstraintType::Rotation:
		Constraint = Controller.AllocateConstraintT<UTickableRotationConstraint>(BaseName, bUseDefault);
		break;
	case ETransformConstraintType::Scale:
		Constraint = Controller.AllocateConstraintT<UTickableScaleConstraint>(BaseName, bUseDefault);
		break;
	case ETransformConstraintType::Parent:
		Constraint = Controller.AllocateConstraintT<UTickableParentConstraint>(BaseName, bUseDefault);
		break;
	case ETransformConstraintType::LookAt:
		Constraint = Controller.AllocateConstraintT<UTickableLookAtConstraint>(BaseName, bUseDefault);
		break;
	default:
		ensure(false);
		break;
	}
	return Constraint;
}

namespace Private
{

// we suppose that both InParentHandle and InChildHandle are safe to use
bool HasConstraintDependencyWith(UWorld* InWorld, const UTransformableHandle* InParentHandle, const UTransformableHandle* InChildHandle)
{
	if (!InWorld || !InParentHandle || !InChildHandle)
	{
		return false;
	}
	
	using ConstraintPtr = TWeakObjectPtr<UTickableConstraint>;
	using HandlePtr = TObjectPtr<UTransformableHandle>;

	static constexpr bool bSorted = false;
	const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(InWorld);
	const TArray<ConstraintPtr> Constraints = Controller.GetParentConstraints(InParentHandle->GetHash(), bSorted);

	// get parent handles
	TArray< HandlePtr > ParentHandles;
	for (const ConstraintPtr& Constraint: Constraints)
	{
		if (const UTickableTransformConstraint* TransformConstraint = Cast<UTickableTransformConstraint>(Constraint.Get()))
		{
			if (IsValid(TransformConstraint->ParentTRSHandle))
			{
				ParentHandles.Add(TransformConstraint->ParentTRSHandle);
			}
		}
	}

	// check if InChildHandle is one of them
	const uint32 ChildHash = InChildHandle->GetHash();
	const bool bIsParentADependency = ParentHandles.ContainsByPredicate([ChildHash](const HandlePtr& InHandle)
	{
		return InHandle->GetHash() == ChildHash;
	});

	if (bIsParentADependency)
	{
		return true;
	}

	// if not, recurse
	for (const HandlePtr& ParentHandle: ParentHandles)
	{
		if (HasConstraintDependencyWith(InWorld, ParentHandle, InChildHandle))
		{
			return true;
		}
	}

	return false;
}
	
bool AreHandlesConstrainable( UWorld* InWorld, const UTransformableHandle* InParentHandle, const UTransformableHandle* InChildHandle)
{
	if (InChildHandle->GetHash() == InParentHandle->GetHash())
	{
		const FText Notification =
			NSLOCTEXT("TransformConstraintUtil", "AreHandlesConstrainable_SameObject", "Cannot create a constraint on itself.");
		NotifyError(Notification);
		
		return false;
	}

	// check for direct transform dependencies (ei hierarchy)
	if (InParentHandle->HasDirectDependencyWith(*InChildHandle))
	{
#if WITH_EDITOR
		FFormatNamedArguments Args;
		Args.Add(TEXT("Parent"), FText::FromString(InParentHandle->GetLabel()));
		Args.Add(TEXT("Child"), FText::FromString(InChildHandle->GetLabel()));
		const FText Notification = FText::Format(NSLOCTEXT("TransformConstraintUtil", "AreHandlesConstrainable_DirectDependency",
			"Cannot create a new constraint between '{Parent}' and '{Child}' as there is a direct dependency between them."), Args);
		NotifyError(Notification);
#endif
		return false;
	}

	// check for indirect transform dependencies (ei constraint chain)
	if (HasConstraintDependencyWith(InWorld, InParentHandle, InChildHandle))
	{
#if WITH_EDITOR
		FFormatNamedArguments Args;
		Args.Add(TEXT("Parent"), FText::FromString(InParentHandle->GetLabel()));
		Args.Add(TEXT("Child"), FText::FromString(InChildHandle->GetLabel()));
		
		const FText Notification = FText::Format(NSLOCTEXT("TransformConstraintUtil", "AreHandlesConstrainable_IndirectDependency",
			"Cannot create a new constraint between '{Parent}' and '{Child}' as there is an indirect dependency between them."), Args);
		NotifyError(Notification);
#endif
		return false;
	}

	return true;
}

}

UTickableTransformConstraint* CreateAndAddFromObjects(
	UWorld* InWorld,
	UObject* InParent, const FName& InParentSocketName,
	UObject* InChild, const FName& InChildSocketName,
	const ETransformConstraintType InType,
	const bool bMaintainOffset,
	const bool bUseDefault,
	const TFunction<void()>& InValidDependencyFunction)
{
	// SANITY CHECK
	if (!InWorld || !InParent || !InChild)
	{
		const FText Notification =
			NSLOCTEXT("TransformConstraintUtil", "CreateAndAddFromObjects_SanityCheck", "Cannot create a new constraint with invalid world on handles.");
		Private::NotifyError(Notification);
		return nullptr;
	}

	UTransformableHandle* ParentHandle = Private::GetHandle(InParent, InParentSocketName);
	if (!ParentHandle)
	{
		return nullptr;
	}
	
	UTransformableHandle* ChildHandle = Private::GetHandle(InChild, InChildSocketName);
	if (!ChildHandle)
	{
		return nullptr;
	}

	const bool bCanConstrain = Private::AreHandlesConstrainable(InWorld, ParentHandle, ChildHandle);
	if (!bCanConstrain)
	{
		ChildHandle->MarkAsGarbage();
		ParentHandle->MarkAsGarbage();
		return nullptr;
	}

	if (InValidDependencyFunction)
	{
		InValidDependencyFunction();
	}
	
	UTickableTransformConstraint* Constraint = CreateFromType(InWorld, InType, bUseDefault);
	if (Constraint && (ParentHandle->IsValid() && ChildHandle->IsValid()))
	{
		if (AddConstraint(InWorld, ParentHandle, ChildHandle, Constraint, bMaintainOffset, bUseDefault) == false)
		{
			Constraint->MarkAsGarbage();
			Constraint = nullptr;
		}
	}
	return Constraint;
}

bool AddConstraint(UWorld* InWorld, UTransformableHandle* InParentHandle, UTransformableHandle* InChildHandle,
	UTickableTransformConstraint* InNewConstraint, const bool bMaintainOffset, const bool bUseDefault)
{
	const bool bIsValidParent = InParentHandle && InParentHandle->IsValid();
	const bool bIsValidChild = InChildHandle && InChildHandle->IsValid();
	
	if (!bIsValidParent || !bIsValidChild)
	{
		const FText Notification =
			NSLOCTEXT("TransformConstraintUtil", "AreHandlesConstrainable_InvalidHandles", "Cannot create a new constraint between invalid handle(s).");
		Private::NotifyError(Notification);
		
		return false;
	}

	if (InNewConstraint == nullptr)
	{
		const FText Notification =
			NSLOCTEXT("TransformConstraintUtil", "AreHandlesConstrainable_InvalidConstraint", "Cannot add a null constraint.");
		Private::NotifyError(Notification);
		
		return false;
	}

	// set handles before calling AddConstraint as it will build dependencies based on them
	InNewConstraint->ParentTRSHandle = InParentHandle;
	InNewConstraint->ChildTRSHandle = InChildHandle;

	// add the new one
	const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(InWorld);
	const bool bConstraintAdded = Controller.AddConstraint(InNewConstraint);

	if (!bConstraintAdded)
	{
		InNewConstraint->ParentTRSHandle = nullptr;
		InNewConstraint->ChildTRSHandle = nullptr;
		return false;
	}

	bool bSetup = !bUseDefault; 
	if (bUseDefault)
	{
		if (UTickableTransformConstraint* CDO = InNewConstraint->GetClass()->GetDefaultObject<UTickableTransformConstraint>())
		{
			bSetup = CDO->bUseCurrentOffset;
		}
	}
	else
	{
		InNewConstraint->bMaintainOffset = bMaintainOffset;
	}

	if (bSetup)
	{
		InNewConstraint->Setup();
	}
	InNewConstraint->InitConstraint(InWorld);
	
	return true;
}

void UpdateTransformBasedOnConstraint(FTransform& InOutCurrentTransform, const USceneComponent* InSceneComponent)
{
	if (AActor* Actor = InSceneComponent->GetTypedOuter<AActor>())
	{
		TArray< TWeakObjectPtr<UTickableConstraint> > Constraints;
		GetParentConstraints(InSceneComponent->GetWorld(), Actor, Constraints);

		const int32 LastActiveIndex = GetLastActiveConstraintIndex(Constraints);
		if (Constraints.IsValidIndex(LastActiveIndex))
		{
			// switch to constraint space
			const FTransform WorldTransform = InSceneComponent->GetSocketTransform(InSceneComponent->GetAttachSocketName());
			const TOptional<FTransform> RelativeTransform = GetConstraintsRelativeTransform(Constraints, InOutCurrentTransform, WorldTransform);
			if (RelativeTransform)
			{
				InOutCurrentTransform = *RelativeTransform;
			}
		}
	}
}

FTransform ComputeRelativeTransform( const FTransform& InChildLocal, const FTransform& InChildWorld, const FTransform& InSpaceWorld,
	const UTickableTransformConstraint* InConstraint)
{
	if (!InConstraint)
	{
		return InChildWorld.GetRelativeTransform(InSpaceWorld);
	}
	
	const ETransformConstraintType ConstraintType = static_cast<ETransformConstraintType>(InConstraint->GetType());
	switch (ConstraintType)
	{
	case ETransformConstraintType::Translation:
		{
			FTransform RelativeTransform = InChildLocal;
			FVector RelativeTranslation = InChildWorld.GetLocation() - InSpaceWorld.GetLocation();
			if (const UTickableTranslationConstraint* TranslationConstraint = Cast<UTickableTranslationConstraint>(InConstraint))
			{
				TranslationConstraint->AxisFilter.FilterVector(RelativeTranslation, InChildLocal.GetTranslation());
			}
			RelativeTransform.SetLocation(RelativeTranslation);
			return RelativeTransform;
		}
	case ETransformConstraintType::Rotation:
		{
			FTransform RelativeTransform = InChildLocal;
			FQuat RelativeRotation = InSpaceWorld.GetRotation().Inverse() * InChildWorld.GetRotation();
			RelativeRotation.Normalize();
			if (const UTickableRotationConstraint* RotationConstraint = Cast<UTickableRotationConstraint>(InConstraint))
			{
				RotationConstraint->AxisFilter.FilterQuat(RelativeRotation, InChildLocal.GetRotation());
			}
			RelativeTransform.SetRotation(RelativeRotation);
			return RelativeTransform;
		}
	case ETransformConstraintType::Scale:
		{
			FTransform RelativeTransform = InChildLocal;
			const FVector SpaceScale = InSpaceWorld.GetScale3D();
			FVector RelativeScale = InChildWorld.GetScale3D();
			RelativeScale[0] = FMath::Abs(SpaceScale[0]) > KINDA_SMALL_NUMBER ? RelativeScale[0] / SpaceScale[0] : 0.f;
			RelativeScale[1] = FMath::Abs(SpaceScale[1]) > KINDA_SMALL_NUMBER ? RelativeScale[1] / SpaceScale[1] : 0.f;
			RelativeScale[2] = FMath::Abs(SpaceScale[2]) > KINDA_SMALL_NUMBER ? RelativeScale[2] / SpaceScale[2] : 0.f;
			if (const UTickableScaleConstraint* ScaleConstraint = Cast<UTickableScaleConstraint>(InConstraint))
			{
				ScaleConstraint->AxisFilter.FilterVector(RelativeScale, InChildLocal.GetScale3D());
			}
			RelativeTransform.SetScale3D(RelativeScale);
			return RelativeTransform;
		}
	case ETransformConstraintType::Parent:
		{
			const UTickableParentConstraint* ParentConstraint = Cast<UTickableParentConstraint>(InConstraint);
			const bool bScale = ParentConstraint ? ParentConstraint->IsScalingEnabled() : true;
			
			FTransform ChildTransform = InChildWorld;
			if (!bScale)
			{
				ChildTransform.RemoveScaling();
			}

			FTransform RelativeTransform = ChildTransform.GetRelativeTransform(InSpaceWorld);

			if (ParentConstraint && !ParentConstraint->TransformFilter.TranslationFilter.HasNoEffect())
			{
				FVector RelativeLocation = RelativeTransform.GetLocation();
				ParentConstraint->TransformFilter.TranslationFilter.FilterVector(RelativeLocation, InChildLocal.GetLocation());
				RelativeTransform.SetLocation(RelativeLocation);
			}

			if (ParentConstraint && !ParentConstraint->TransformFilter.RotationFilter.HasNoEffect())
			{
				FQuat RelativeRotation = RelativeTransform.GetRotation();
				ParentConstraint->TransformFilter.RotationFilter.FilterQuat(RelativeRotation, InChildLocal.GetRotation());
				RelativeTransform.SetRotation(RelativeRotation);
			}

			if (ParentConstraint && !ParentConstraint->TransformFilter.ScaleFilter.HasNoEffect())
			{
				FVector RelativeScale = RelativeTransform.GetScale3D();
				ParentConstraint->TransformFilter.ScaleFilter.FilterVector(RelativeScale, InChildLocal.GetScale3D());
				RelativeTransform.SetScale3D(RelativeScale);
			}
			
			if (!bScale)
			{
				RelativeTransform.SetScale3D(InChildLocal.GetScale3D());
			}
			return RelativeTransform;
		}
	case ETransformConstraintType::LookAt:
		return InChildLocal;
	default:
		break;
	}
	
	return InChildWorld.GetRelativeTransform(InSpaceWorld);
}

TOptional<FTransform> GetRelativeTransform(UWorld* InWorld, const uint32 InHandleHash)
{
	if (!InWorld || InHandleHash <= 0)
	{
		return TOptional<FTransform>();
	}

	static constexpr bool bSorted = true;
	const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(InWorld);

	const TArray< TWeakObjectPtr<UTickableConstraint> > Constraints = Controller.GetParentConstraints(InHandleHash, bSorted);
	if (Constraints.IsEmpty())
	{
		return TOptional<FTransform>();
	}

	// get current active transform constraint 	
	const int32 LastActiveIndex = GetLastActiveConstraintIndex(Constraints);
	if (!Constraints.IsValidIndex(LastActiveIndex))
	{
		return TOptional<FTransform>();
	}

	// get relative transform
	const UTickableTransformConstraint* Constraint = Cast<UTickableTransformConstraint>(Constraints[LastActiveIndex].Get());
	const FTransform ChildLocal = Constraint->GetChildLocalTransform();
	const FTransform ChildGlobal = Constraint->GetChildGlobalTransform();
	
	return GetConstraintsRelativeTransform(Constraints, ChildLocal, ChildGlobal);
}

TOptional<FTransform> GetConstraintsRelativeTransform(
	const TArray< TWeakObjectPtr<UTickableConstraint> >& InConstraints,
	const FTransform& InChildLocal, const FTransform& InChildWorld)
{
	if (InConstraints.IsEmpty())
	{
		return TOptional<FTransform>();
	}

	// get current active transform constraint
	const int32 LastActiveIndex = GetLastActiveConstraintIndex(InConstraints);
	if (!InConstraints.IsValidIndex(LastActiveIndex))
	{
		return TOptional<FTransform>();
	}

	// get relative transform
	// if that constraint handles the entire transform then return the relative transform directly
	const UTickableTransformConstraint* Constraint = Cast<UTickableTransformConstraint>(InConstraints[LastActiveIndex].Get());
	if (EnumHasAllFlags(Constraint->GetChannelsToKey(), EMovieSceneTransformChannel::AllTransform))
	{
		const FTransform ParentGlobal = Constraint->GetParentGlobalTransform();
		return ComputeRelativeTransform(InChildLocal, InChildWorld, ParentGlobal, Constraint);
	}

	// otherwise, we need to look for constraints on a sub-transform basis so we compute the relative transform for each of them
	using ConstraintPtr = TWeakObjectPtr<UTickableConstraint>;
	auto GetLastSubTransformIndex = [InConstraints](const EMovieSceneTransformChannel& InChannel)
	{
		return InConstraints.FindLastByPredicate([InChannel](const ConstraintPtr& InConstraint)
		{
			if (const UTickableTransformConstraint* TransformConstraint = Cast<UTickableTransformConstraint>(InConstraint.Get()))
			{
				const bool bHasChannelFlag = EnumHasAllFlags(TransformConstraint->GetChannelsToKey(), InChannel);
				return InConstraint->IsFullyActive() && TransformConstraint->bDynamicOffset && bHasChannelFlag;
			}
			return false;
		});
	};

	// look for last constraint index for each channel 
	static constexpr EMovieSceneTransformChannel SubChannels[3] = {	EMovieSceneTransformChannel::Translation,
																	EMovieSceneTransformChannel::Rotation,
																	EMovieSceneTransformChannel::Scale };
	TArray<int32> SubTransformIndices;
	for (int32 Index = 0; Index < 3; Index++)
	{
		const int32 LastSubIndex = GetLastSubTransformIndex(SubChannels[Index]);
    	if (InConstraints.IsValidIndex(LastSubIndex))  
    	{
    		SubTransformIndices.AddUnique(LastSubIndex);
    	}
	}
	SubTransformIndices.Sort();

	// if none then return 
	if (SubTransformIndices.IsEmpty())
	{
		return TOptional<FTransform>();
	}

	// iterate thru constraints to compute the relative transform in each of them 
	FTransform ChildLocal = InChildLocal;
	for (const int32 SubConstraintIndex: SubTransformIndices)
	{
		const UTickableTransformConstraint* SubConstraint = Cast<UTickableTransformConstraint>(InConstraints[SubConstraintIndex].Get());
		const FTransform ParentGlobal = SubConstraint->GetParentGlobalTransform();
		ChildLocal = ComputeRelativeTransform(ChildLocal, InChildWorld, ParentGlobal, SubConstraint);
	}
	return ChildLocal;
}

int32 GetLastActiveConstraintIndex(const TArray< TWeakObjectPtr<UTickableConstraint> >& InConstraints)
{
	return InConstraints.FindLastByPredicate([](const TWeakObjectPtr<UTickableConstraint>& InConstraint)
	{
	   if (const UTickableTransformConstraint* TransformConstraint = Cast<UTickableTransformConstraint>(InConstraint.Get()))
	   {
		   return InConstraint->Active && TransformConstraint->bDynamicOffset;
	   }
	   return false;
	});
}

void GetChildrenConstraints( UWorld* World, const UTickableTransformConstraint* InConstraint,
	TArray< TWeakObjectPtr<UTickableConstraint> >& OutConstraints,
	const bool bIncludeTarget)
{
	if (!InConstraint || !InConstraint->IsValid())
	{
		// this probably has been checked before but we want to make sure the data is safe to use
		return;
	}
	
	const UTransformableHandle* Handle = InConstraint->ChildTRSHandle.Get();
	
	using ConstraintPtr = TWeakObjectPtr<UTickableConstraint>;
	
	// filter for transform constraints where the InHandle is the parent (based on its hash value)
	// and also has the same target if bIncludeTarget is true
	const uint32 ParentHash = Handle->GetHash();
	const UObject* ParentTarget = Handle->GetTarget().Get();
	auto Predicate = [InConstraint, Handle, ParentHash, bIncludeTarget, ParentTarget, World](const ConstraintPtr& Constraint)
	{
		const UTickableTransformConstraint* TransformConstraint = Cast<UTickableTransformConstraint>(Constraint.Get());
		if (!TransformConstraint || TransformConstraint == InConstraint)
		{
			return false;
		}

		if (TransformConstraint->ParentTRSHandle)
		{
			const UTransformableHandle* OtherParentHandle = TransformConstraint->ParentTRSHandle;
			if (OtherParentHandle->GetHash() == ParentHash)
			{
				return true;
			}

			if (bIncludeTarget && ParentTarget)
			{
				const UObject* Target = OtherParentHandle->GetTarget().Get();
				if (Target == ParentTarget)
				{
					// check direct dependencies to avoid evaluation order issues
					if (Handle->HasDirectDependencyWith(*OtherParentHandle))
					{
						return false;
					}

					// check constraints dependencies to avoid cycles
					if (Private::HasConstraintDependencyWith(World, OtherParentHandle, Handle))
					{
						return false;
					}

					// check TransformConstraint's ChildHandle
					if (const UTransformableHandle* OtherChildHandle = TransformConstraint->ChildTRSHandle)
					{
						const UTransformableHandle* ParentHandle = InConstraint->ParentTRSHandle.Get();
						
						// if TransformConstraint's child is InConstraint's parent then avoid cycles
						if (OtherChildHandle->GetHash() == ParentHandle->GetHash())
						{
							return false;
						}

						// check dependencies with OtherChildHandle to avoid cycles
						const FHandleDependencyChecker Checker(World);
						if (Checker.HasDependency(*ParentHandle, *OtherChildHandle))
						{
							return false;
						}
						
						// check dependencies with OtherParentHandle to avoid cycles
						if (Checker.HasDependency(*ParentHandle, *OtherParentHandle))
						{
							return false;
						}
					}
					
					return true;
				}
			}
		}

		return false;
	};

	const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(World);
	const TArray<ConstraintPtr> FilteredConstraints = Controller.GetConstraintsByPredicate(Predicate);
	OutConstraints.Append(FilteredConstraints);
}

UTickableTransformConstraint* GetConfig(const UClass* InConstraintClass)
{
	UConstraintSubsystem* Subsystem = UConstraintSubsystem::Get();
	return Subsystem && InConstraintClass ? Cast<UTickableTransformConstraint>(Subsystem->GetConfig(InConstraintClass)) : nullptr;
}
	
}
