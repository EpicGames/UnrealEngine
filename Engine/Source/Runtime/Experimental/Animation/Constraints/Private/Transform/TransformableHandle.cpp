// Copyright Epic Games, Inc. All Rights Reserved.


#include "Transform/TransformableHandle.h"
#include "ConstraintsManager.h"
#include "Components/SceneComponent.h"
#include "GameFramework/Actor.h"
#include "Engine/Engine.h"
#include "EntitySystem/MovieSceneSharedPlaybackState.h"
#include "MovieSceneSection.h"
#include "Transform/TransformableHandleUtils.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Components/SkeletalMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TransformableHandle)

/**
 * UTransformableHandle
 */

UTransformableHandle::~UTransformableHandle()
{
	OnHandleModified.Clear();
}

UTransformableHandle::FHandleModifiedEvent& UTransformableHandle::HandleModified()
{
	return OnHandleModified;
}

void UTransformableHandle::Notify(EHandleEvent InEvent, const bool bPreTickTarget) const
{
	if (!bNotifying && OnHandleModified.IsBound())
	{
		TGuardValue<bool> ReentrantGuardSelf(bNotifying, true);
		if (bPreTickTarget)
		{
			TickTarget();
		}
		
		OnHandleModified.Broadcast(const_cast<UTransformableHandle*>(this), InEvent);
	}
}

bool UTransformableHandle::HasBoundObjects() const
{
	return ConstraintBindingID.IsValid();
}

void UTransformableHandle::OnBindingIDsUpdated(const TMap<UE::MovieScene::FFixedObjectBindingID, UE::MovieScene::FFixedObjectBindingID>& OldFixedToNewFixedMap, FMovieSceneSequenceID LocalSequenceID, TSharedRef<UE::MovieScene::FSharedPlaybackState> SharedPlaybackState)
{
	if (ConstraintBindingID.IsValid())
	{
		UE::MovieScene::FFixedObjectBindingID FixedBindingID = ConstraintBindingID.ResolveToFixed(LocalSequenceID, SharedPlaybackState);
		if (OldFixedToNewFixedMap.Contains(FixedBindingID))
		{
			Modify();

			const FMovieSceneSequenceHierarchy* Hierarchy = SharedPlaybackState->GetHierarchy();
			ConstraintBindingID = OldFixedToNewFixedMap[FixedBindingID].ConvertToRelative(LocalSequenceID, Hierarchy);
		}
	}
}

void UTransformableHandle::PreEvaluate(const bool bTick) const
{
	if (bTick)
	{
		TickTarget();
	}
}

/**
 * UTransformableComponentHandle
 */

UTransformableComponentHandle::~UTransformableComponentHandle()
{
	UnregisterDelegates();
}

void UTransformableComponentHandle::PostLoad()
{
	Super::PostLoad();
	RegisterDelegates();
}

bool UTransformableComponentHandle::IsValid(const bool bDeepCheck) const
{
	return Component.IsValid();
}

//need to tick any skelmesh component, sibling or parent
void UTransformableComponentHandle::TickTarget() const
{
	if (!Component.IsValid())
	{
		return;
	}
	
	if (TransformableHandleUtils::SkipTicking())
	{
		if (SocketName != NAME_None)
		{
			TransformableHandleUtils::EvaluateComponent(Component.Get()); 
		}
	}
	else
	{
		TransformableHandleUtils::TickDependantComponents(Component.Get());	
	}
}

void UTransformableComponentHandle::PreEvaluate(const bool bTickSkeletalMeshes) const
{
	if (!Component.IsValid())
	{
		return;
	}
	
	if (TransformableHandleUtils::SkipTicking())
	{
		if (SocketName != NAME_None)
		{
			TransformableHandleUtils::EvaluateComponent(Component.Get());
		}
	}
}

void UTransformableComponentHandle::SetGlobalTransform(const FTransform& InGlobal) const
{
	if (Component.IsValid())
	{
		const FVector OldScale = GetGlobalTransform().GetScale3D();
		const FVector NewScale = InGlobal.GetScale3D();
		const bool bMarkRenderDirty =
			FMath::Sign(OldScale[0]) != FMath::Sign(NewScale[0]) ||
			FMath::Sign(OldScale[1]) != FMath::Sign(NewScale[1]) ||
			FMath::Sign(OldScale[2]) != FMath::Sign(NewScale[2]);
		
		Component->SetWorldTransform(InGlobal);
		
		if (Component->GetWorld())
		{
			FConstraintsManagerController::OnSceneComponentConstrained().Broadcast(Component.Get());
		}

		if (bMarkRenderDirty)
		{
			Component->MarkRenderStateDirty();
		}
	}
}

void UTransformableComponentHandle::SetLocalTransform(const FTransform& InLocal) const
{
	if(Component.IsValid())
	{
		Component->SetRelativeTransform(InLocal);
		if (Component->GetWorld())
		{
			FConstraintsManagerController::OnSceneComponentConstrained().Broadcast(Component.Get());
		}
	}
}

FTransform UTransformableComponentHandle::GetLocalTransform() const
{
	return Component.IsValid() ? Component->GetRelativeTransform() : FTransform::Identity;
}

FTransform UTransformableComponentHandle::GetGlobalTransform() const
{
	USceneComponent* SceneComponent = Component.Get();
	if (!SceneComponent)
	{
		return FTransform::Identity;
	}
	
	if (TransformableHandleUtils::SkipTicking())
	{
		constexpr bool bValidForTransforms = true;
		const UE::Anim::FAnimationEvaluator& AnimationEvaluator = TransformableHandleUtils::EvaluateComponent(SceneComponent);
		if (AnimationEvaluator.IsValid(bValidForTransforms))
		{
			return AnimationEvaluator.GetGlobalTransform(SocketName);
		}
		
		return TransformableHandleUtils::EvaluateGlobalTransform(SceneComponent, SocketName);
	}
	
	return Component.IsValid() ? Component->GetSocketTransform(SocketName) : FTransform::Identity;
}

UObject* UTransformableComponentHandle::GetPrerequisiteObject() const
{
	return Component.Get(); 
}

FTickFunction* UTransformableComponentHandle::GetTickFunction() const
{
	return Component.IsValid() ? &Component->PrimaryComponentTick : nullptr;
}

uint32 UTransformableComponentHandle::GetHash() const
{
	return Component.IsValid() ? GetTypeHash(Component.Get()) : 0;
}

TWeakObjectPtr<UObject> UTransformableComponentHandle::GetTarget() const
{
	return Component;
}

bool UTransformableComponentHandle::HasDirectDependencyWith(const UTransformableHandle& InOther) const
{
	const uint32 OtherHash = InOther.GetHash();
	if (OtherHash == 0)
	{
		return false;
	}

	if (Component.IsValid())
	{
		// check whether the other handle is one of the component's parent
		for (const USceneComponent* Comp=Component->GetAttachParent(); Comp!=nullptr; Comp=Comp->GetAttachParent() )
		{
			const uint32 AttachParentHash = GetTypeHash(Comp);
			if (AttachParentHash == OtherHash)
			{
				return true;
			}
		}
	}

	return false;
}

namespace
{
	
FTickPrerequisite LookForPrimaryPrerequisite(USceneComponent* Component, const bool bAllowThis)
{
	static const FTickPrerequisite DummyPrerex;
	
	if (!Component)
	{
		return DummyPrerex;
	}

	auto IsValidTickFunction = [](const FTickFunction* InTickFunction)
	{
		if (InTickFunction)
		{
			return (InTickFunction->bCanEverTick || InTickFunction->IsTickFunctionRegistered());
		};

		return false;
	};

	// check if this can tick
	if (IsValidTickFunction(&Component->PrimaryComponentTick) && bAllowThis)
	{
		return FTickPrerequisite(Component, Component->PrimaryComponentTick);
	}

	// check if a parent constraint can tick
	if (UWorld* World = Component->GetWorld())
	{
		static constexpr bool bSorted = true;
		
		const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(World);
		const TArray<TWeakObjectPtr<UTickableConstraint>> ParentConstraints =
			Controller.GetParentConstraints(GetTypeHash(Component), bSorted);
		
		for (int Index = ParentConstraints.Num()-1; Index >= 0; Index--)
		{
			if (UTickableConstraint* Constraint = ParentConstraints[Index].Get())
			{
				if(IsValidTickFunction(&Constraint->GetTickFunction(World)))
				{
					return FTickPrerequisite(Constraint->GetOuter(), Constraint->GetTickFunction(World));
				}
			}
		}	
	}

	// check if parent ticks
	if (USceneComponent* Parent = Component->GetAttachParent())
	{
		FTickFunction& ParentTickFunction = Parent->PrimaryComponentTick;
		if (IsValidTickFunction(&ParentTickFunction))
		{
			return FTickPrerequisite(Parent, ParentTickFunction);
		}

		// get up the hierarchy if not found
		constexpr bool bAllowTickFromParent = true;
		return LookForPrimaryPrerequisite(Parent, bAllowTickFromParent);
	}

	return DummyPrerex;
}
	
}

FTickPrerequisite UTransformableComponentHandle::GetPrimaryPrerequisite(const bool bAllowThis) const
{
	if (Component.IsValid())
	{
		return LookForPrimaryPrerequisite(Component.Get(), bAllowThis);
	}

	static const FTickPrerequisite DummyPrerex;
	return DummyPrerex;
}

void UTransformableComponentHandle::UnregisterDelegates() const
{
#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
	if (GEngine)
	{
		GEngine->OnActorMoving().RemoveAll(this);
	}

	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);
#endif
}

void UTransformableComponentHandle::RegisterDelegates()
{
	UnregisterDelegates();

#if WITH_EDITOR
	if (GEngine)
	{
		// NOTE BINDER: this has to be done before binding UTransformableComponentHandle::OnActorMoving
		if (!GEngine->OnActorMoving().IsBoundToObject(&GetEvaluationBinding()))
		{
			GEngine->OnActorMoving().AddRaw(&GetEvaluationBinding(), &FComponentEvaluationGraphBinding::OnActorMoving);
		}
		
		GEngine->OnActorMoving().AddUObject(this, &UTransformableComponentHandle::OnActorMoving);
	}
	FCoreUObjectDelegates::OnObjectPropertyChanged.AddUObject(this, &UTransformableComponentHandle::OnPostPropertyChanged);
	FCoreUObjectDelegates::OnObjectsReplaced.AddUObject(this, &UTransformableComponentHandle::OnObjectsReplaced);
#endif
}

#if WITH_EDITOR
void UTransformableComponentHandle::OnActorMoving(AActor* InActor)
{
	if (!Component.IsValid())
	{
		return;
	}

	const TInlineComponentArray<USceneComponent*> Components(InActor);
	if (!Components.Contains(Component))
	{
		return;
	}

	GetEvaluationBinding().bPendingFlush = true;
	
	Notify(EHandleEvent::GlobalTransformUpdated);
}

void UTransformableComponentHandle::OnPostPropertyChanged(
	UObject* InObject,
	FPropertyChangedEvent& InPropertyChangedEvent)
{
	if (!Component.IsValid())
	{
		return;
	}

	USceneComponent* SceneComponent = Cast<USceneComponent>(InObject);
	if (!SceneComponent)
	{
		if (const AActor* Actor = Cast<AActor>(InObject))
		{
			const TInlineComponentArray<USceneComponent*> Components(Actor);
			const int32 Index = Components.IndexOfByKey(Component.Get());
			if (Index != INDEX_NONE)
			{
				SceneComponent = Components[Index];
			}
		}
	}
	
	if (SceneComponent != Component)
	{
		return;
	}

	const FProperty* MemberProperty = InPropertyChangedEvent.MemberProperty;
	if (!MemberProperty)
	{
		return;
	}
	
	const FName MemberPropertyName = MemberProperty->GetFName();
	const bool bTransformationChanged =
		(MemberPropertyName == USceneComponent::GetRelativeLocationPropertyName() ||
			MemberPropertyName == USceneComponent::GetRelativeRotationPropertyName() ||
			MemberPropertyName == USceneComponent::GetRelativeScale3DPropertyName());
	if (!bTransformationChanged)
	{
		return;
	}

	Notify(EHandleEvent::GlobalTransformUpdated);
}

void UTransformableComponentHandle::OnObjectsReplaced(const TMap<UObject*, UObject*>& InOldToNewInstances)
{
	// in the context of blueprints being recompiled (cf. AActor::RerunConstructionScripts()), the component has to
	// be updated. as this is called after it has been destroyed, we get the component even if it's pending kill
	// otherwise Get() will return a nullptr.
	static constexpr bool bEvenIfPendingKill = true;
	if (UObject* NewObject = InOldToNewInstances.FindRef(Component.Get(bEvenIfPendingKill)))
	{
		if (USceneComponent* NewSceneComponent= Cast<USceneComponent>(NewObject))
		{
			Component = NewSceneComponent;
			RegisterDelegates();
		}
	}
}
#endif

TArrayView<FMovieSceneFloatChannel*>  UTransformableComponentHandle::GetFloatChannels(const UMovieSceneSection* InSection) const
{
	// no floats for transform sections
	static const TArrayView<FMovieSceneFloatChannel*> EmptyChannelsView;
	return EmptyChannelsView;
}

TArrayView<FMovieSceneDoubleChannel*>  UTransformableComponentHandle::GetDoubleChannels(const UMovieSceneSection* InSection) const
{
	static const TArrayView<FMovieSceneDoubleChannel*> EmptyChannelsView;
	if (InSection)
	{
		return InSection->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();
	}
	return EmptyChannelsView;
}

bool UTransformableComponentHandle::AddTransformKeys(const TArray<FFrameNumber>& Frames,
	const TArray<FTransform>& InLocalTransforms,
	const EMovieSceneTransformChannel& InChannels,
	const FFrameRate& InTickResolution,
	UMovieSceneSection* InTransformSection,
	const bool bLocal) const
{
	//todo 
	//note this is the same as MOvieSceneToolHelpers::AddTransformKeys but that's in the editor code, need to find a place in the runtime someewhere
	if (!InTransformSection)
	{
		return false;
	}

	const int32 NumFrames = Frames.Num();
	const int32 NumTransforms = InLocalTransforms.Num();

	const bool bUpdateDefault = NumFrames == 0 && NumTransforms == 1;
	if (NumFrames == 0)
	{
		if (!bUpdateDefault)
		{
			return false;
		}
	}
	else if (NumTransforms != NumFrames)
	{
		return false;
	}

	auto GetValue = [](const uint32 Index, const FVector& InLocation, const FRotator& InRotation, const FVector& InScale)
	{
		switch (Index)
		{
		case 0:
			return InLocation.X;
		case 1:
			return InLocation.Y;
		case 2:
			return InLocation.Z;
		case 3:
			return InRotation.Roll;
		case 4:
			return InRotation.Pitch;
		case 5:
			return InRotation.Yaw;
		case 6:
			return InScale.X;
		case 7:
			return InScale.Y;
		case 8:
			return InScale.Z;
		default:
			ensure(false);
			break;
		}
		return 0.0;
	};

	const bool bKeyTranslation = EnumHasAllFlags(InChannels, EMovieSceneTransformChannel::Translation);
	const bool bKeyRotation = EnumHasAllFlags(InChannels, EMovieSceneTransformChannel::Rotation);
	const bool bKeyScale = EnumHasAllFlags(InChannels, EMovieSceneTransformChannel::Scale);

	TArray<uint32> ChannelsIndexToKey;
	if (bKeyTranslation)
	{
		ChannelsIndexToKey.Append({ 0,1,2 });
	}
	if (bKeyRotation)
	{
		ChannelsIndexToKey.Append({ 3,4,5 });
	}
	if (bKeyScale)
	{
		ChannelsIndexToKey.Append({ 6,7,8 });
	}

	const TArrayView<FMovieSceneDoubleChannel*> Channels =
		InTransformSection->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();

	// set default
	const FTransform& LocalTransform0 = InLocalTransforms[0];
	const FVector Location0 = LocalTransform0.GetLocation();
	const FRotator Rotation0 = LocalTransform0.GetRotation().Rotator();
	const FVector Scale3D0 = LocalTransform0.GetScale3D();

	for (int32 ChannelIndex = 0; ChannelIndex < 9; ChannelIndex++)
	{
		if (Channels.IsValidIndex(ChannelIndex))
		{
			FMovieSceneDoubleChannel* Channel = Channels[ChannelIndex];
			if (!Channel->GetDefault().IsSet() || (bUpdateDefault && Channel->GetNumKeys() == 0))
			{
				const double Value = GetValue(ChannelIndex, Location0, Rotation0, Scale3D0);
				Channel->SetDefault(Value);
			}
		}
	}

	if (NumFrames > 0)
	{
		// add keys
		for (int32 Index = 0; Index < Frames.Num(); ++Index)
		{
			const FFrameNumber& Frame = Frames[Index];
			const FTransform& LocalTransform = InLocalTransforms[Index];

			const FVector Location = LocalTransform.GetLocation();
			const FRotator Rotation = LocalTransform.GetRotation().Rotator();
			const FVector Scale3D = LocalTransform.GetScale3D();

			for (const int32 ChannelIndex : ChannelsIndexToKey)
			{
				if (Channels.IsValidIndex(ChannelIndex))
				{
					const double Value = GetValue(ChannelIndex, Location, Rotation, Scale3D);
					TMovieSceneChannelData<FMovieSceneDoubleValue> ChannelData = Channels[ChannelIndex]->GetData();
					{
						int32 ExistingIndex = ChannelData.FindKey(Frame);
						if (ExistingIndex != INDEX_NONE)
						{
							FMovieSceneDoubleValue& DoubleValue = ChannelData.GetValues()[ExistingIndex];
							DoubleValue.Value = Value;
						}
						else
						{
							FMovieSceneDoubleValue NewKey(Value);
							ERichCurveTangentWeightMode WeightedMode = RCTWM_WeightedNone;
							NewKey.InterpMode = ERichCurveInterpMode::RCIM_Cubic;
							NewKey.TangentMode = ERichCurveTangentMode::RCTM_Auto;
							NewKey.Tangent.ArriveTangent = 0.0f;
							NewKey.Tangent.LeaveTangent = 0.0f;
							NewKey.Tangent.TangentWeightMode = WeightedMode;
							NewKey.Tangent.ArriveTangentWeight = 0.0f;
							NewKey.Tangent.LeaveTangentWeight = 0.0f;
							ChannelData.AddKey(Frame, NewKey);
						}
					}
				}
			}
		}

		//now we need to set auto tangents
		for (const int32 ChannelIndex : ChannelsIndexToKey)
		{
			if (Channels.IsValidIndex(ChannelIndex))
			{
				Channels[ChannelIndex]->AutoSetTangents();
			}
		}
	}
	return true;
}

void UTransformableComponentHandle::ResolveBoundObjects(FMovieSceneSequenceID LocalSequenceID, TSharedRef<UE::MovieScene::FSharedPlaybackState> SharedPlaybackState, UObject* SubObject)
{
	// in the context of blueprints being recompiled (cf. AActor::RerunConstructionScripts()) or with spawnable objects, the component has to be updated.
	// as this can be called after it has been destroyed, we get the component even if it's pending kill otherwise Get() will return a nullptr.
	static constexpr bool bEvenIfPendingKill = true;
	const USceneComponent* ComponentEvenIfPendingKill = Component.Get(bEvenIfPendingKill);

	auto ReplaceComponent = [this](USceneComponent* InNewComponent)
	{
		if (InNewComponent != Component)
		{
			Component = InNewComponent;
			RegisterDelegates();
		}
	};
	
	//just do one
	if (auto BoundObjectsView = ConstraintBindingID.ResolveBoundObjects(LocalSequenceID, SharedPlaybackState); BoundObjectsView.Num() > 0)
	{
		const TWeakObjectPtr<>& ParentObject = BoundObjectsView[0];
		if (const AActor* Actor = Cast<AActor>(ParentObject.Get()))
		{
			const TInlineComponentArray<USceneComponent*> Components(Actor);
			const int32 Index = Components.IndexOfByPredicate([this, ComponentEvenIfPendingKill](const USceneComponent* SubComponent)
			{
				return ComponentEvenIfPendingKill && (SubComponent->GetFName() == ComponentEvenIfPendingKill->GetFName());
			});

			ReplaceComponent(Index != INDEX_NONE ? Components[Index] : Actor->GetRootComponent());
		}
		else if (USceneComponent* Comp = Cast<USceneComponent>(ParentObject.Get()))
		{
			ReplaceComponent(Comp);
		}
	}
}

UTransformableHandle* UTransformableComponentHandle::Duplicate(UObject* NewOuter) const
{
	UTransformableComponentHandle* HandleCopy = DuplicateObject<UTransformableComponentHandle>(this, NewOuter);
	HandleCopy->Component = Component;
	HandleCopy->SocketName = SocketName;
	return HandleCopy;
}

#if WITH_EDITOR
FString UTransformableComponentHandle::GetLabel() const
{
	if (!Component.IsValid())
	{
		if (SocketName != NAME_None)
		{
			return FString::Printf(TEXT("(invalid-component)/%s"), *SocketName.ToString());
		}
		return TEXT("(invalid-component)");
	}

	if (SocketName != NAME_None)
	{
		return SocketName.ToString(); 
	}

	const AActor* Actor = Component->GetOwner();
	return Actor ? Actor->GetActorLabel() : Component->GetName();
}

FString UTransformableComponentHandle::GetFullLabel() const
{
	if (!Component.IsValid())
	{
		if (SocketName != NAME_None)
		{
			return FString::Printf(TEXT("(invalid-component)/%s"), *SocketName.ToString());
		}
		return TEXT("(invalid-component)");
	}

	const AActor* Actor = Component->GetOwner();
	const FString ComponentLabel = Actor ? Actor->GetActorLabel() : Component->GetName();
	
	if (SocketName == NAME_None)
	{
		return ComponentLabel;
	}

	return FString::Printf(TEXT("%s/%s"), *ComponentLabel, *SocketName.ToString() );
};

#endif

FComponentEvaluationGraphBinding& UTransformableComponentHandle::GetEvaluationBinding()
{
	static FComponentEvaluationGraphBinding EvaluationBinding;
	return EvaluationBinding;
}

void FComponentEvaluationGraphBinding::OnActorMoving(AActor* InActor)
{
	if (!bPendingFlush)
	{
		return;
	}
	
	if (UWorld* World = InActor ? InActor->GetWorld() : nullptr)
	{
		// flush all pending evaluations if any
		const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(World);
		Controller.FlushEvaluationGraph();
	}
	bPendingFlush = false;
}
