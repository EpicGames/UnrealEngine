// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transform/TransformableHandleUtils.h"

#include "GameFramework/Actor.h"
#include "Components/SceneComponent.h"
#include "Components/SkeletalMeshComponent.h"

namespace TransformableHandleUtils
{

static int32 SkeletalMeshTickingMode = 2;
static bool	bNewEvaluation = true;

static FAutoConsoleVariableRef CVarSkeletalMeshTickingMode(
	TEXT("Constraints.SkeletalMesh.TickingMode"),
	SkeletalMeshTickingMode,
	TEXT( "Constraint skeletal mesh ticking mode ([0, 2] - default: 2):\n" )
		TEXT( "0 - do not tick any related skeletal mesh\n" )
		TEXT( "1 - only tick the constrained skeletal mesh\n")
		TEXT( "2 - tick all the skeletal meshes attached to the constrained actor\n"),
		FConsoleVariableDelegate::CreateLambda([](IConsoleVariable*)
	{
		SkeletalMeshTickingMode = FMath::Clamp(SkeletalMeshTickingMode, 0, 2);
	})
);

static FAutoConsoleVariableRef CVarNewEvaluation(
	TEXT("Constraints.NewEvaluation"),
	bNewEvaluation,
	TEXT("Set constraints new evaluation scheme."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable*)
	{
		UE_LOGF(LogTemp, Warning, "Constraints' new evaluation scheme %ls", bNewEvaluation ? TEXT("enabled.") : TEXT("disabled."));
		SkeletalMeshTickingMode = bNewEvaluation ? 0 : 2;
	})
);

bool SkipTicking()
{
	return bNewEvaluation;
}

void TickDependantComponents(USceneComponent* InComponent)
{
	if (SkeletalMeshTickingMode <= 0 || bNewEvaluation)
	{
		return;
	}
	
	if (!InComponent)
	{
		return;
	}

	if (SkeletalMeshTickingMode == 1)
	{
		return TickSkeletalMeshComponent(Cast<USkeletalMeshComponent>(InComponent));
	}

	static constexpr bool bIncludeFromChildActors = true;

	const AActor* Parent = InComponent->GetOwner();
	while (Parent)
	{
		Parent->ForEachComponent<USkeletalMeshComponent>(bIncludeFromChildActors, &TickSkeletalMeshComponent);
		Parent = Parent->GetAttachParentActor();
	}
}

void TickSkeletalMeshComponent(USkeletalMeshComponent* InSkeletalMeshComponent)
{
	if (!InSkeletalMeshComponent || bNewEvaluation)
	{
		return;
	}

	// avoid re-entrant animation evaluation
	if (InSkeletalMeshComponent->IsPostEvaluatingAnimation())
	{
		return;
	}

	static constexpr float DeltaTime = 0.03f;
	static constexpr bool bNeedsValidRootMotion = false;
	
	InSkeletalMeshComponent->TickAnimation(DeltaTime, bNeedsValidRootMotion);
	InSkeletalMeshComponent->RefreshBoneTransforms();
	InSkeletalMeshComponent->RefreshFollowerComponents();
	InSkeletalMeshComponent->UpdateComponentToWorld();
	InSkeletalMeshComponent->FinalizeBoneTransform();
	InSkeletalMeshComponent->MarkRenderTransformDirty();
	InSkeletalMeshComponent->MarkRenderDynamicDataDirty();
}
	
void MarkComponentForEvaluation(const USceneComponent* InSceneComponent, const bool bRecursive)
{
	if (InSceneComponent)
	{
		if (const USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(InSceneComponent))
		{
			UE::Anim::FAnimationEvaluationCache::Get().MarkForEvaluation(SkeletalMeshComponent);
		}
		
		if (bRecursive)
		{
			USceneComponent* Parent = InSceneComponent->GetAttachParent();
			while (Parent)
			{
				MarkComponentForEvaluation(Parent);
				Parent = Parent->GetAttachParent();
			}
		}
	}
}

FTransform GetGlobalTransform(USceneComponent* InSceneComponent, const FName InSocketName)
{
	return EvaluateGlobalTransform(InSceneComponent, InSocketName);
}
	
FTransform EvaluateGlobalTransform(USceneComponent* InSceneComponent, const FName InSocketName)
{
	if (!InSceneComponent)
	{
		return FTransform::Identity;
	}

	constexpr bool bValidForTransforms = true;
	const UE::Anim::FAnimationEvaluator& AnimationEvaluator = EvaluateComponent(InSceneComponent);
	if (AnimationEvaluator.IsValid(bValidForTransforms))
	{
		return AnimationEvaluator.GetGlobalTransform(InSocketName);
	}

	if (!InSceneComponent->GetAttachParent())
	{
		return InSceneComponent->GetSocketTransform(InSocketName);
	}
	
	const bool bHasSkeletalMeshInHierarchy = [](const USceneComponent* Component)
	{
		while (Component)
		{
			if (Component->IsA<USkeletalMeshComponent>())
			{
				return true;
			}
			Component = Component->GetAttachParent();
		}
		return false;
	}(InSceneComponent);

	if (!bHasSkeletalMeshInHierarchy)
	{
		return InSceneComponent->GetSocketTransform(InSocketName);
	}

	FTransform Global = InSceneComponent->GetSocketTransform(InSocketName, RTS_Component) * InSceneComponent->GetRelativeTransform();

	USceneComponent* Parent = InSceneComponent->GetAttachParent();
	FName AttachSocket = InSceneComponent->GetAttachSocketName();
	while (Parent)
	{
		USkeletalMeshComponent* ParentSkeletalMeshComponent = Cast<USkeletalMeshComponent>(Parent);

		bool bFromEvaluator = false;
		if (ParentSkeletalMeshComponent && AttachSocket != NAME_None)
		{
			const UE::Anim::FAnimationEvaluator& ParentEvaluator = EvaluateComponent(ParentSkeletalMeshComponent);
			if (ParentEvaluator.IsValid(bValidForTransforms))
			{
				Global = Global * ParentEvaluator.GetRelativeTransform(AttachSocket);
				bFromEvaluator = true;
			}
		}

		if (!bFromEvaluator)
		{
			FTransform Socket = Parent->GetSocketTransform(AttachSocket, RTS_Component) * Parent->GetRelativeTransform();
			Global = Global * Socket;
		}

		AttachSocket = Parent->GetAttachSocketName();
		Parent = Parent->GetAttachParent();
	}

	return Global;
}

FTransform GetGlobalTransform(const USceneComponent* InSceneComponent, const FName InSocketName)
{
	using namespace UE::Anim;
	
	if (!InSceneComponent)
	{
		return FTransform::Identity;
	}

	const USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(InSceneComponent);	
	
	constexpr bool bValidForTransforms = true;
	FAnimationEvaluationCache& EvaluationCache = FAnimationEvaluationCache::Get();
	const FAnimationEvaluator& Evaluator = EvaluationCache.FindEvaluator(SkeletalMeshComponent);
	if (SkeletalMeshComponent && !Evaluator.IsValid(bValidForTransforms))
	{
		return InSceneComponent->GetSocketTransform(InSocketName);
	}
	
	const bool bShouldUseEvaluators = [&EvaluationCache, InSceneComponent]()
	{
		bool bUseEvaluators = false;
		
		const USceneComponent* Parent = InSceneComponent->GetAttachParent();
		FName AttachSocket = InSceneComponent->GetAttachSocketName();
		
		while (Parent)
		{
			const USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(Parent);
			if (SkeletalMeshComponent && AttachSocket != NAME_None)
			{
				constexpr bool bValidForTransforms = true;
				const FAnimationEvaluator& Evaluator = EvaluationCache.FindEvaluator(SkeletalMeshComponent);
				bUseEvaluators |= Evaluator.IsValid(bValidForTransforms);
				if (!bUseEvaluators)
				{
					return false;
				}
			}
			
			AttachSocket = Parent->GetAttachSocketName();
			Parent = Parent->GetAttachParent();
		}
			
		return bUseEvaluators;
	}();

	if (!bShouldUseEvaluators)
	{
		return InSceneComponent->GetSocketTransform(InSocketName);
	}

	FTransform Global = SkeletalMeshComponent && Evaluator.IsValid(bValidForTransforms) ?
		Evaluator.GetRelativeTransform(InSocketName) :
		InSceneComponent->GetSocketTransform(InSocketName, RTS_Component) * InSceneComponent->GetRelativeTransform();
	
	const USceneComponent* Parent = InSceneComponent->GetAttachParent();
	FName AttachSocket = InSceneComponent->GetAttachSocketName();
	while (Parent)
	{
		const USkeletalMeshComponent* ParentSkeletalMeshComponent = Cast<USkeletalMeshComponent>(Parent);

		bool bFromEvaluator = false;
		if (ParentSkeletalMeshComponent && AttachSocket != NAME_None)
		{
			const FAnimationEvaluator& ParentEvaluator = EvaluationCache.FindEvaluator(ParentSkeletalMeshComponent);
			if (!ParentEvaluator.IsValid(bValidForTransforms))
			{
				return InSceneComponent->GetSocketTransform(InSocketName);
			}
			Global = Global * ParentEvaluator.GetRelativeTransform(AttachSocket);
			bFromEvaluator = true;
		}

		if (!bFromEvaluator)
		{
			FTransform Socket = Parent->GetSocketTransform(AttachSocket, RTS_Component) * Parent->GetRelativeTransform();
			Global = Global * Socket;
		}

		AttachSocket = Parent->GetAttachSocketName();
		Parent = Parent->GetAttachParent();
	}

	return Global;
}

const UE::Anim::FAnimationEvaluator& EvaluateComponent(USceneComponent* InSceneComponent)
{
	if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(InSceneComponent))
	{
		return UE::Anim::FAnimationEvaluationCache::Get().GetEvaluator(SkeletalMeshComponent);
	}

	static const UE::Anim::FAnimationEvaluator Invalid(nullptr);
	return Invalid;
}

const UE::Anim::FAnimationEvaluator& EvaluateComponent(USceneComponent* InSceneComponent, const UE::Anim::FAnimationEvaluationTask& InTask)
{
	if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(InSceneComponent))
	{
		return UE::Anim::FAnimationEvaluationCache::Get().GetEvaluator(SkeletalMeshComponent, InTask);
	}

	static const UE::Anim::FAnimationEvaluator Invalid(nullptr);
	return Invalid;
}

}