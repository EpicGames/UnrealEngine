// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transform/TransformableRegistry.h"

#include "Transform/TransformableHandle.h"
#include "Transform/TransformConstraintUtil.h"
#include "GameFramework/Actor.h"
#include "Components/SceneComponent.h"
#include "UObject/Class.h"

FTransformableRegistry::~FTransformableRegistry() = default;

FTransformableRegistry& FTransformableRegistry::Get()
{
	static FTransformableRegistry Singleton;
	return Singleton;
}

void FTransformableRegistry::Register(UClass* InClass, CreateHandleFuncT InHandleFunc, GetHashFuncT InHashFunc)
{
	const FTransformableInfo Info { InHandleFunc, InHashFunc};
	Transformables.Emplace(InClass, Info);	
}

FTransformableRegistry::GetHashFuncT FTransformableRegistry::GetHashFunction(const UClass* InClass) const
{
	if (InClass)
	{
		if (const FTransformableInfo* Info = FindInfo(InClass))
		{
			return Info->GetHashFunc; 
		}
	}
	return {};
}

FTransformableRegistry::CreateHandleFuncT FTransformableRegistry::GetCreateFunction(const UClass* InClass) const
{
	if (InClass)
	{
		if(const FTransformableInfo* Info = FindInfo(InClass))
		{
			return Info->CreateHandleFunc; 
		}
	}
	return {};
}

const FTransformableRegistry::FTransformableInfo* FTransformableRegistry::FindInfo(const UClass* InClass) const
{
	if (!InClass)
	{
		return nullptr;
	}
	
	// look for registered class
	if (const FTransformableInfo* Info = Transformables.Find(InClass))
	{
		return Info;
	}

	// if not directly registered, look for super class registration
	return FindInfo(InClass->GetSuperClass());
}

void FTransformableRegistry::RegisterBaseObjects()
{
    // register USceneComponent and AActor
    auto CreateComponentHandle = [](UObject* InObject, const FName& InSocketName)->UTransformableHandle*
    {
    	if (USceneComponent* Component = Cast<USceneComponent>(InObject))
    	{
    		return UE::TransformConstraintUtil::CreateHandleForSceneComponent(Component, InSocketName);
    	}
    	return nullptr;
    };

    auto GetComponentHash = [](const UObject* InObject, const FName&)->uint32
    {
    	if (const USceneComponent* Component = Cast<USceneComponent>(InObject))
    	{
    		return GetTypeHash(Component);
    	}
    	return 0;
    };
    
    auto CreateComponentHandleFromActor = [CreateComponentHandle](UObject* InObject, const FName& InSocketName)->UTransformableHandle*
    {
    	if (const AActor* Actor = Cast<AActor>(InObject))
    	{
    		return CreateComponentHandle(Actor->GetRootComponent(), InSocketName);
    	}
    	return nullptr;
    };

    auto GetComponentHashFromActor = [GetComponentHash](const UObject* InObject, const FName& InSocketName)->uint32
    {
    	if (const AActor* Actor = Cast<AActor>(InObject))
    	{
    		return GetComponentHash(Actor->GetRootComponent(), InSocketName);
    	}
    	return 0;
    };

	FTransformableRegistry& Registry = Get();
    Registry.Register(USceneComponent::StaticClass(), CreateComponentHandle, GetComponentHash);
    Registry.Register(AActor::StaticClass(), CreateComponentHandleFromActor, GetComponentHashFromActor);
}

void FTransformableRegistry::UnregisterAllObjects()
{
	FTransformableRegistry& Registry = Get();
	Registry.Transformables.Reset();
}