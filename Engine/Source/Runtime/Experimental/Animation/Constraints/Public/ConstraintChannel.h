// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneBoolChannel.h"
#include "Channels/MovieSceneChannelEditorData.h"
#include "Channels/MovieSceneChannelTraits.h"
#include "ConstraintsManager.h"
#include "Containers/UnrealString.h"
#include "Misc/FrameTime.h"
#include "MovieSceneClipboard.h"
#include "Templates/Function.h"
#include "UObject/Class.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/SoftObjectPtr.h"

#include "ConstraintChannel.generated.h"

USTRUCT()
struct FMovieSceneConstraintChannel : public FMovieSceneBoolChannel
{
	GENERATED_BODY()

	FMovieSceneConstraintChannel() {};

	/**
	 * Access a mutable interface for this channel's data
	 *
	 * @return An object that is able to manipulate this channel's data
	 */
	virtual TMovieSceneChannelData<bool> GetData() override
	{
		return TMovieSceneChannelData<bool>(&Times, &Values, this, &KeyHandles);
	}

	/**
	 * Access a constant interface for this channel's data
	 *
	 * @return An object that is able to interrogate this channel's data
	 */
	virtual TMovieSceneChannelData<const bool> GetData() const override
	{
		return FMovieSceneBoolChannel::GetData();
	}
	
	/**
	 * Evaluate this channel
	 *
	 * @param InTime     The time to evaluate at
	 * @param OutValue   A value to receive the result
	 * @return true if the channel was evaluated successfully, false otherwise
	 */
	CONSTRAINTS_API virtual bool Evaluate(FFrameTime InTime, bool& OutValue) const override;

	/**
	 * Checks whether this constraint channel is infinite
	 * @return true if there are no keys and the default value is set to true.
	 */
	CONSTRAINTS_API bool IsInfinite() const;
	
#if WITH_EDITOR
	using ExtraLabelFunction = TFunction< FString() >;
	ExtraLabelFunction ExtraLabel;
#endif
};

template<>
struct TStructOpsTypeTraits<FMovieSceneConstraintChannel> : public TStructOpsTypeTraitsBase2<FMovieSceneConstraintChannel>
{
	enum { WithStructuredSerializeFromMismatchedTag = true };
};


template<>
struct TMovieSceneChannelTraits<FMovieSceneConstraintChannel> : TMovieSceneChannelTraitsBase<FMovieSceneConstraintChannel>
{
	enum { SupportsDefaults = false };

#if WITH_EDITOR

	/** Byte channels can have external values (ie, they can get their values from external objects for UI purposes) */
	typedef TMovieSceneExternalValue<bool> ExtendedEditorDataType;

#endif
};

// #if WITH_EDITOR
// namespace MovieSceneClipboard
// {
// 	template<> inline FName GetKeyTypeName<bool>()
// 	{
// 		static FName Name("Bool");
// 		return Name;
// 	}
// }
// #endif


USTRUCT()
struct FConstraintAndActiveChannel
{
	GENERATED_USTRUCT_BODY()

	FConstraintAndActiveChannel() {}
	FConstraintAndActiveChannel(const TObjectPtr<UTickableConstraint>& InConstraint)
		: ConstraintCopyToSpawn(InConstraint)
	{};
	TObjectPtr<UTickableConstraint> GetConstraint() const { return ConstraintCopyToSpawn; }
	void SetConstraint(UTickableConstraint* InConstraint) { ConstraintCopyToSpawn = InConstraint; }

	UPROPERTY()
	FMovieSceneConstraintChannel ActiveChannel;

private:
	UPROPERTY()
	TObjectPtr<UTickableConstraint> ConstraintCopyToSpawn;
};
