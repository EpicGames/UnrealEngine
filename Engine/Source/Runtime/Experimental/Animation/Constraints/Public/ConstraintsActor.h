// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "ConstraintsActor.generated.h"

class UConstraintsManager;
class UObject;

UCLASS(NotPlaceable, MinimalAPI)
class AConstraintsActor : public AActor
{
	GENERATED_BODY()

public:
	// Sets default values for this actor's properties
	CONSTRAINTS_API AConstraintsActor(const FObjectInitializer& ObjectInitializer);
	CONSTRAINTS_API virtual ~AConstraintsActor();

	CONSTRAINTS_API virtual void BeginDestroy() override;
	CONSTRAINTS_API virtual void Destroyed() override;

	CONSTRAINTS_API virtual void PostRegisterAllComponents() override;

#if WITH_EDITOR
	CONSTRAINTS_API virtual void PostEditUndo() override;
#endif
	
protected:
	// Called when the game starts or when spawned
	CONSTRAINTS_API virtual void BeginPlay() override;

public:
	// Called every frame
	CONSTRAINTS_API virtual void Tick(float DeltaTime) override;

	UPROPERTY()
	TObjectPtr<UConstraintsManager> ConstraintsManager = nullptr;
	
private:
	void RegisterConstraintsTickFunctions() const;
};
