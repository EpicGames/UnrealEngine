// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UTickableConstraint;
struct FConstraintTickFunction;
class USceneComponent;

#define UE_API CONSTRAINTS_API

/**
 * FSequentialConstraintsEvaluator provides a sequential constraint evaluation scheme, ensuring that parents are marked for evaluation only if necessary.
 * NOTE: this scheme is only effective if UseFastEvaluation() is true.
 */

struct FSequentialConstraintsEvaluator 
{
	FSequentialConstraintsEvaluator(USceneComponent* InInitialComponent = nullptr);
	void DoEvaluate(UTickableConstraint* InConstraint, const FConstraintTickFunction* InConstraintTickFunction = nullptr);
	
private:
	USceneComponent* LastMarkedForEvaluation = nullptr;
};

namespace UE::Anim::Constraints
{
	/* Whether the fast sequential evaluation is enabled. */
	bool UseFastEvaluation();

	/* Evaluate all the constraints sequentially. this will use a faster evaluation scheme UseFastEvaluation() is true. */
	void UE_API EvaluateConstraints(TConstArrayView<TWeakObjectPtr<UTickableConstraint>> InConstraints);
}

#undef UE_API