// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConstraintsEvaluation.h"
#include "UObject/Object.h"
#include "Engine/EngineBaseTypes.h"

struct FConstraintTickFunction;
class UTickableConstraint;
class UWorld;
struct FConstraintsInWorld;

/**
 * A structure that uniquely identifies a constraint within the evaluation graph
 */

struct FConstraintNode
{
	/* Constraint data */
	FGuid ConstraintID;
	FConstraintTickFunction* ConstraintTick = nullptr;

	/* Internal indices for quick navigation through the nodes and constraints data structures */
	int32 NodeIndex = INDEX_NONE;
	int32 ConstraintIndex = INDEX_NONE;

	/* internal hierarchy representing dependencies at the nodes level (indices here represent the nodes' NodeIndex)*/
	TSet<uint32> Parents;
	TSet<uint32> Children;

	/* Whether this node needs to be evaluated on the next flush. */
	bool bMarkedForEvaluation = false;
	
	/* Whether this node is currently being evaluated (used to avoid re-entrant evaluations). */
	bool bEvaluating = false;
};

/**
 * A graph like structure to efficiently represent constraints' evaluation hierarchy using their tick dependencies.
 * Each node stores the useful data to evaluate the constraint hierarchically. 
 */

struct FConstraintsEvaluationGraph: public TSharedFromThis<FConstraintsEvaluationGraph>
{
public:
	explicit FConstraintsEvaluationGraph(const FConstraintsInWorld* InConstraintsInWorld)
		: ConstraintsInWorld(*InConstraintsInWorld)
	{}

	/**
	* Marks the constraint for evaluation. Note that it will only be evaluated after FlushPendingEvaluations has been called.
	* This makes it possible to tag several constraints and evaluate the graph only once.
	*/
	void MarkForEvaluation(const TWeakObjectPtr<UTickableConstraint>& InConstraint);
	
	/* Evaluates constraints marked for evaluation + their dependencies. */
	CONSTRAINTS_API void FlushPendingEvaluations();
	
	/* Invalidates internal data so that the graph will be rebuilt on request. */
	void InvalidateData();

	/* Rebuilds the full graph taking constraints tick dependencies into account. */
	void Rebuild();

	/* Should the graph be evaluated ? */
	bool IsPendingEvaluation() const;

	/* Get Sorted Constraints */
	bool GetSortedConstraints(TArray<TWeakObjectPtr<UTickableConstraint>>& OutConstraints);
	
private:

	enum EGraphState
	{
		InvalidData = 0,	// Invalid data
		ReadyForEvaluation,	// Internal data has been built and is ready for evaluation
		PendingEvaluation,	// Some constraints has been marked for evaluation
		Flushing			// The graph is currently being evaluated
	};
	
	FConstraintsEvaluationGraph(FConstraintsEvaluationGraph&& InOther) = default;
	FConstraintsEvaluationGraph(const FConstraintsEvaluationGraph& InOther) = delete;
	FConstraintsEvaluationGraph& operator=(const FConstraintsEvaluationGraph& InOther) = delete;
	
	/* Get or create a new node in the graph that wraps that constraint. */
	FConstraintNode& GetNode(const TWeakObjectPtr<UTickableConstraint>& InConstraint);

	/* Returns the node wrapping that constraint if any. */
	FConstraintNode* FindNode(const TWeakObjectPtr<UTickableConstraint>& InConstraint);

	/* Evaluates InNode if its tick function is enabled + evaluates its children recursively. */
	void Evaluate(FConstraintNode* InNode);

	/* Dumps the current state of the graph. */
	void Dump() const;

	/* Array of constraint wrappers. */
	TArray<FConstraintNode> Nodes;

	/* References to the actual stored data. */
	const FConstraintsInWorld& ConstraintsInWorld;
	
	/* Current graph state. */
	EGraphState	State = InvalidData;

	/* Constraints sequential evaluator used if the fast evaluation scheme is enabled. */
	FSequentialConstraintsEvaluator SequentialEvaluator;
};

namespace UE::Constraints::Graph
{
	
using ConstraintPtr = TWeakObjectPtr<UTickableConstraint>;

/** Builds a directed graph from a constraints array using the tick prerequisites to define dependencies.
* @param InWorld - The UWorld in which tick functions are registered.
* @param InConstraints - An array of constraints.
* @param OutNodes - A ordered data structure storing constraints' dependencies. 
*/
void BuildGraph(UWorld* InWorld, const TArrayView<const ConstraintPtr>& InConstraints, TArray<FConstraintNode>& OutNodes);

/** Sorts the constraint array by building a directed graph from it, using the tick prerequisites to define dependencies.
* @param InWorld - The UWorld in which tick functions are registered.
* @param InOutConstraints - An array of constraints that will be re-ordered from the constraints' dependencies.   
*/
void SortConstraints(UWorld* InWorld, TArray<ConstraintPtr>& InOutConstraints);

}