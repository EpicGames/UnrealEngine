// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "Engine/EngineBaseTypes.h"
#include "MovieSceneSequenceID.h"
#include "Misc/Guid.h"
#include "Engine/Level.h"
#include "ConstraintsManager.generated.h"

class UTickableConstraint;

namespace UE::MovieScene { struct FSharedPlaybackState; }

/** 
 * FConstraintTickFunction
 * Represents the interface of constraint as a tick function. This allows both to evaluate a constraint in the
 * UE ticking system but also to handle dependencies between parents/children and constraints between themselves
 * using the tick prerequisites system.
 **/

USTRUCT()
struct FConstraintTickFunction : public FTickFunction
{
	GENERATED_USTRUCT_BODY()
public:
	CONSTRAINTS_API FConstraintTickFunction();
	CONSTRAINTS_API ~FConstraintTickFunction();

	CONSTRAINTS_API FConstraintTickFunction(const FConstraintTickFunction& In);
	/* Begin FTickFunction Interface */
	CONSTRAINTS_API virtual void ExecuteTick(
		float DeltaTime,
		ELevelTick TickType,
		ENamedThreads::Type CurrentThread,
		const FGraphEventRef& MyCompletionGraphEvent) override;
	
	CONSTRAINTS_API virtual FString DiagnosticMessage() override;
	/* End FTickFunction Interface */

	/** Callable function that represents the actual constraint. **/
	using ConstraintFunction = TFunction<void()>;
	
	/** Register a callable function. **/
	CONSTRAINTS_API void RegisterFunction(ConstraintFunction InConstraint);
	
	/** Register a callable function. **/
	CONSTRAINTS_API void EvaluateFunctions() const;

	/** Weak ptr to the Constraint holding this tick function. **/
	TWeakObjectPtr<UTickableConstraint> Constraint;
	
	/** The list of the constraint functions that will be called within the tick function. **/
	TArray<ConstraintFunction> ConstraintFunctions;
};

template<>
struct TStructOpsTypeTraits<FConstraintTickFunction> : public TStructOpsTypeTraitsBase2<FConstraintTickFunction>
{
	enum
	{
		WithCopy = false
	};
};

/** 
 * UTickableConstraint
 * Represents the basic interface of constraint within the constraints manager.
 **/

UCLASS(Abstract, Blueprintable, MinimalAPI)
class UTickableConstraint : public UObject
{
	GENERATED_BODY()
	
public:
	UTickableConstraint() {}
	virtual ~UTickableConstraint() = default;

	/** Returns the actual function that the tick function needs to evaluate. */
	virtual FConstraintTickFunction::ConstraintFunction GetFunction() const PURE_VIRTUAL(GetFunction, return {};);

	/** Evaluates the constraint in a context where it's mot done thru the ConstraintTick's tick function. */
	CONSTRAINTS_API virtual void Evaluate(bool bTickHandlesAlso = false) const;

	/** Whether or not it's valid for example it may not be fully loaded, missing handles or set to not be valid*/
	virtual bool IsValid(const bool bDeepCheck = true) const PURE_VIRTUAL(IsValid, return false;);
	/** Sets the Active value and enable/disable the tick function. */
	CONSTRAINTS_API virtual void SetActive(const bool bIsActive);

	/** Get whether or not it's fully active, it's set to active and all pieces are set up,e.g. objects really exist that are being constrainted*/
	CONSTRAINTS_API virtual bool IsFullyActive() const;
	/** If true it contains objects bound to an external system, like sequencer so we don't do certain things, like remove constraints when they don't resolve*/
	virtual bool HasBoundObjects() const PURE_VIRTUAL(HasBoundObjects, return false;);
	/** Resolve the bound objects so that any object it references are resovled and correctly set up*/
	virtual void ResolveBoundObjects(FMovieSceneSequenceID LocalSequenceID, TSharedRef<UE::MovieScene::FSharedPlaybackState> SharedPlaybackState, UObject* SubObject = nullptr)  PURE_VIRTUAL(ResolveBoundObjects);

	/** @todo document */
	virtual uint32 GetTargetHash() const PURE_VIRTUAL(GetTargetHash, return 0;);
	/** Whether or not this object references this object. If it HasBoundObjects you should ResolveBoundObjects if you expect this to be up to date */
	virtual bool ReferencesObject(TWeakObjectPtr<UObject> InObject) const PURE_VIRTUAL(ReferencesObject, return false;);
	/** Create duplicate with new Outer*/
	CONSTRAINTS_API virtual UTickableConstraint* Duplicate(UObject* NewOuter) const;
	/** Initialize and register the constraint with that world*/
	virtual void InitConstraint(UWorld * InWorld)  PURE_VIRTUAL(UTickableConstraint::InitConstraint);
	/** Teardown and unregister the constraint with that world*/
	virtual void TeardownConstraint(UWorld* InWorld)  PURE_VIRTUAL(UTickableConstraint::DeInitConstraint);
	/** */
	virtual void AddedToWorld(UWorld* InWorld) PURE_VIRTUAL(UTickableConstraint::AddedToWorld);
	
	/** Get Tick function in that world*/
	CONSTRAINTS_API FConstraintTickFunction& GetTickFunction(const UWorld* InWorld);
	CONSTRAINTS_API const FConstraintTickFunction& GetTickFunction(const UWorld* InWorld) const;

	/** Returns the tick function registrered in the world's current level if any (assuming both world and current leve are valid) */
	FConstraintTickFunction* FindTickFunction(const UWorld* InWorld) const;

	// UObject interface
	CONSTRAINTS_API virtual void PostLoad() override;
	CONSTRAINTS_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
	CONSTRAINTS_API virtual void PostInitProperties() override;
	// End of UObject interface
	
#if WITH_EDITOR
	/** Returns the constraint's label used for UI. */
	CONSTRAINTS_API virtual FString GetLabel() const;
	CONSTRAINTS_API virtual FString GetFullLabel() const;

	/** Returns the constraint's type label used for UI. */
	CONSTRAINTS_API virtual FString GetTypeLabel() const;

	// UObject editor only interface
	CONSTRAINTS_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	CONSTRAINTS_API virtual void PostEditUndo() override;
	// End of UObject editor only interface
	
#endif
	/** @todo documentation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, DisplayName="Active State", Category="Constraint")
	bool Active = true;

	UPROPERTY()
	bool bValid = false;


	/** Unique id to be used to fine constraint in a particular world, may be shared*/
	FGuid ConstraintID;

protected:

	/** Tick function that will be registered and evaluated. */
	mutable TMap<TWeakObjectPtr<ULevel>, TUniquePtr<FConstraintTickFunction>, TInlineSetAllocator<4>> ConstraintTicks;
};


/** 
 * UConstraintsManager
 * This object gathers the different static/nonanimated constraints of the level and is held by the ConstraintsActor (unique in the level)
 * Note in 5.4 all of the constraints are owned by the subsystem, so need to get that to get at animated constraints
 **/

UCLASS(BLUEPRINTABLE, MinimalAPI)
class UConstraintsManager : public UObject
{
	GENERATED_BODY()
public:

	using ConstraintPtr = TObjectPtr<UTickableConstraint>;

	/** Dynamic blueprintable delegates for knowing when a constraints are added or deleted*/
	DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_TwoParams(FOnConstraintAdded, UConstraintsManager, OnConstraintAdded_BP, UConstraintsManager*, Mananger, UTickableConstraint*, Constraint);
	DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_ThreeParams(FOnConstraintRemoved, UConstraintsManager, OnConstraintRemoved_BP, UConstraintsManager*, Mananger, UTickableConstraint*, Constraint, bool, bDoNotCompensate);

	
	CONSTRAINTS_API UConstraintsManager();
	CONSTRAINTS_API virtual ~UConstraintsManager();
	
	//UObjects
	CONSTRAINTS_API virtual void	PostLoad() override;

	CONSTRAINTS_API void Init(UWorld* InWorld);
	CONSTRAINTS_API void Clear(UWorld* World);

	void RemoveStaticConstraint(UTickableConstraint* InConstraint) { Constraints.Remove(InConstraint); }
	bool IsStaticConstraint(UTickableConstraint* InConstraint) const { return Constraints.Contains(InConstraint); }

	/** Find the existing Constraints Manager. */
	static CONSTRAINTS_API UConstraintsManager* Find(const UWorld* InWorld);

private:
	/** Get the existing Constraints Manager if existing or create a new one. */
	static CONSTRAINTS_API UConstraintsManager* Get(UWorld* InWorld);


public:
	/** BP Delegate fired when constraints are added*/
	UPROPERTY(BlueprintAssignable, Category = Constraints, meta = (DisplayName = "OnConstraintAdded"))
	FOnConstraintAdded OnConstraintAdded_BP;
	
	/** BP Delegate fired when constraints are removed*/
	UPROPERTY(BlueprintAssignable, Category = Constraints, meta = (DisplayName = "OnConstraintRemoved"))
	FOnConstraintRemoved OnConstraintRemoved_BP;

private:


	/** @todo document */
	FDelegateHandle OnActorDestroyedHandle;
	CONSTRAINTS_API void OnActorDestroyed(AActor* InActor);

	CONSTRAINTS_API void RegisterDelegates(UWorld* World);
	CONSTRAINTS_API void UnregisterDelegates(UWorld* World);


	/** @todo document */
	CONSTRAINTS_API void Dump() const;

	/**  For 5.4, this now just contains all of the static constraints.
	Other constraints will exist in the subsystem and owned by the section*/
	UPROPERTY()
	TArray< TObjectPtr<UTickableConstraint> > Constraints;

	friend class AConstraintsActor;
	friend class FConstraintsManagerController;
};

/** 
 * FConstraintsManagerController
 * Basic controller to add / remove / get constraints
 **/

enum class EConstraintsManagerNotifyType
{
	ConstraintAdded,					/** A new constraint has been added. */
	ConstraintRemoved,					/** A constraint has been removed. */
	ConstraintRemovedWithCompensation,	/** A constraint has been removed and needs compensation. */
	ManagerUpdated,						/** The manager has been updated/reset. */
	GraphUpdated						/** The evaluation graph has been updated/reset. */
};

DECLARE_MULTICAST_DELEGATE_TwoParams(FConstraintsManagerNotifyDelegate, EConstraintsManagerNotifyType /* type */, UObject* /* subject */);

class FConstraintsManagerController
{
public:
	/** Static to control if we should remove constraints or not, may not want to do some when
	compensating since we may be deleting spawnables while doing so*/
	static CONSTRAINTS_API bool bDoNotRemoveConstraint;
	/** Get and Set Controller acive in this world*/
	static CONSTRAINTS_API FConstraintsManagerController& Get(UWorld* InWorld);

	/** Allocates a new constraint with the constraints manager as the owner. */
	template< typename TConstraint >
	TConstraint* AllocateConstraintT(const FName& InBaseName, const bool bUseDefault = false) const;

	/** Add this constraint to the manager */
	CONSTRAINTS_API bool AddConstraint(UTickableConstraint* InConstraint) const;
	
	/** Make a copy of this constraint and add it to the manager, returns the copy mode if it was added*/
	CONSTRAINTS_API UTickableConstraint* AddConstraintFromCopy(UTickableConstraint* CopyOfConstraint) const;

	/** Get the index of the given constraint's ID. */
	CONSTRAINTS_API int32 GetConstraintIndex(const FGuid& InGuid) const;

	/** Remove the constraint at the given index. */
	CONSTRAINTS_API bool RemoveConstraint(const int32 InConstraintIndex, bool bDoNotCompensate = false);

	/** Remove constraint by ptr*/
	CONSTRAINTS_API bool RemoveConstraint(UTickableConstraint* InConstraint, bool bDoNotCompensate = false);
	
	/** Disable a constraint and remove it from the subsystem. */
	CONSTRAINTS_API bool UnregisterConstraint(UTickableConstraint* InConstraint);

	/** Remove All Constraints*/
	CONSTRAINTS_API bool RemoveAllConstraints(bool bDoNotCompensate = false);

	/** Returns the constraint based on it's ID within the manager's constraints array. */
	CONSTRAINTS_API UTickableConstraint* GetConstraint(const FGuid& InGuid) const;

	/** Returns the constraint based on it's index within the manager's constraints array. */
	CONSTRAINTS_API UTickableConstraint* GetConstraint(const int32 InConstraintIndex) const;
	
	/** Get read-only access to the array of constraints. */
	CONSTRAINTS_API const TArray< TWeakObjectPtr<UTickableConstraint> >& GetConstraintsArray() const;

	/** Returns systems' constraints array (sorted if needed). */
	CONSTRAINTS_API TArray< TWeakObjectPtr<UTickableConstraint> > GetAllConstraints(const bool bSorted = false) const;

	/** Returns the static/non-animated constraints*/
	CONSTRAINTS_API TArray< TWeakObjectPtr<UTickableConstraint> > GetStaticConstraints(const bool bSorted = false) const;

	/** Returns a filtered constraints array checking if the predicate for each element is true. */
	template <typename Predicate>
	TArray< TWeakObjectPtr<UTickableConstraint> > GetConstraintsByPredicate(Predicate Pred, const bool bSorted = false) const;

	/** Get parent constraints of the specified child. If bSorted is true, then the constraints will be sorted by dependency. */
	CONSTRAINTS_API TArray< TWeakObjectPtr<UTickableConstraint> > GetParentConstraints(
		const uint32 InTargetHash,
		const bool bSorted = false) const;

	/** Set dependencies between two constraints. */
	CONSTRAINTS_API void SetConstraintsDependencies(
		const FName& InNameToTickBefore,
		const FName& InNameToTickAfter) const;

	CONSTRAINTS_API void SetConstraintsDependencies(
		const FGuid& InGuidToTickBefore,
		const FGuid& InGuidToTickAfter) const;

	/** Go through each constraint in order and evaluate and tick them*/
	CONSTRAINTS_API void EvaluateAllConstraints() const;

	/** when PIEing/Simulating it's possible that the constraint isn't in the active manager but still lives*/
	UE_DEPRECATED(5.7, "Function is no longer needed.")
	CONSTRAINTS_API bool DoesExistInAnyWorld(UTickableConstraint* InConstraint);

	UE_DEPRECATED(5.7, "Function is no longer needed.")
	CONSTRAINTS_API void StaticConstraintCreated(UWorld* InWorld, UTickableConstraint* InConstraint);

	/** If a static constraint is created, we call this, which stores it to the manager/actor*/
	CONSTRAINTS_API void StaticConstraintCreated(UTickableConstraint* InConstraint);

private:
	/** Get the index of the given constraint's name. */
	int32 GetConstraintIndex(const FName& InConstraintName) const;
	
	/** Find the existing Constraints Manager in World or create a new one. */
	CONSTRAINTS_API UConstraintsManager* GetManager();
	
	/** Find the existing Constraints Manager in World. */
	CONSTRAINTS_API UConstraintsManager* FindManager() const;

	/** Destroy the ConstraintsManager from the World. */
	CONSTRAINTS_API void DestroyManager() const;

	/** The World that holds the ConstraintsManagerActor. */
	UWorld* World = nullptr;

public:
	/** Delegate that's fired when a scene component is constrained, this is needed to make sure things like gizmo's get updated after the constraint tick happens*/
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnSceneComponentConstrained, USceneComponent* /*InSceneComponent*/);
	CONSTRAINTS_API static FOnSceneComponentConstrained& OnSceneComponentConstrained();

	/** Delegate to trigger changes in the constraints manager. */
	CONSTRAINTS_API static FConstraintsManagerNotifyDelegate& GetNotifyDelegate();

	/** Notify from changes in the constraints manager. */
	CONSTRAINTS_API static void Notify(EConstraintsManagerNotifyType InNotifyType, UObject* InObject);

	/** Evaluation graph API. */
	CONSTRAINTS_API void MarkConstraintForEvaluation(UTickableConstraint* InConstraint) const;
	CONSTRAINTS_API void InvalidateEvaluationGraph() const;
	CONSTRAINTS_API void FlushEvaluationGraph() const;
};
