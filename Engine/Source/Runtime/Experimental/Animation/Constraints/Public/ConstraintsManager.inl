// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConstraintsManager.h"
#include "ConstraintSubsystem.h"
#include "Algo/StableSort.h"

template< typename TConstraint >
TConstraint* FConstraintsManagerController::AllocateConstraintT(const FName& InBaseName, const bool bUseDefault) const
{
	UConstraintSubsystem* Subsystem =  UConstraintSubsystem::Get();
	if (!Subsystem)
	{
		return nullptr;
	}
	// unique name (we may want to use another approach here to manage uniqueness)
	FName Name = MakeUniqueObjectName(Subsystem, TConstraint::StaticClass(), InBaseName);

	// ensure that the constraint isn't already registered in the ConstraintManager
	const TArray<TWeakObjectPtr<UTickableConstraint>>& Constraints = Subsystem->GetConstraints(World);
	const bool bNameFound = Constraints.ContainsByPredicate([Name](const TWeakObjectPtr<UTickableConstraint>& Constraint)
	{
		return Constraint.IsValid() && Constraint->GetFName() == Name;
	});
	if (bNameFound)
	{
		Name = MakeUniqueObjectName(Subsystem, TConstraint::StaticClass(), InBaseName, EUniqueObjectNameOptions::GloballyUnique);
	}

	UObject* Template = bUseDefault ? Subsystem->GetConfig(TConstraint::StaticClass()) : nullptr;
	TConstraint* NewConstraint = NewObject<TConstraint>(Subsystem, Name, RF_Transactional, Template, Template != nullptr);
	NewConstraint->Modify();
	return NewConstraint;
}

template <typename Predicate>
TArray< TWeakObjectPtr<UTickableConstraint> > FConstraintsManagerController::GetConstraintsByPredicate(
	Predicate Pred, const bool bSorted) const
{
	using ConstraintPtr = TWeakObjectPtr<UTickableConstraint>;
	UConstraintSubsystem* Subsystem = UConstraintSubsystem::Get();

	if (!Subsystem)
	{
		static const TArray< TWeakObjectPtr<UTickableConstraint> > Empty;
		return Empty;
	}

	TArray< ConstraintPtr > Constraints = Subsystem->GetConstraints(World).FilterByPredicate(Pred);
	if (!bSorted)
	{
		return Constraints;
	}

	// LHS ticks before RHS = LHS is a prerex of RHS 
	auto TicksBefore = [this](const UTickableConstraint& LHS, const UTickableConstraint& RHS)
	{
		const TArray<FTickPrerequisite>& RHSPrerex = RHS.GetTickFunction(World).GetPrerequisites();
		const FConstraintTickFunction& LHSTickFunction = LHS.GetTickFunction(World);
		const bool bIsLHSAPrerexOfRHS = RHSPrerex.ContainsByPredicate([&LHSTickFunction](const FTickPrerequisite& Prerex)
		{
			return Prerex.PrerequisiteTickFunction == &LHSTickFunction;
		});
		return bIsLHSAPrerexOfRHS;
	};
	
	Algo::StableSort(Constraints, [TicksBefore](const ConstraintPtr& LHS, const ConstraintPtr& RHS)
	{
		return (LHS.IsValid() && RHS.IsValid()) ? TicksBefore(*LHS, *RHS) : false;
	});
	
	return Constraints;
}