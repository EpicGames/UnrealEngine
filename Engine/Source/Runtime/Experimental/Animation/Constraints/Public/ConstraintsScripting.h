// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Constraint.h"
#include "ConstraintsScripting.generated.h"

class UConstraintsManager;
class UTransformableHandle;
class UTransformableComponentHandle;
class UTickableConstraint;
class UTickableTransformConstraint;

/**
* This is a set of helper functions to access various parts of the Sequencer and Control Rig API via Python and Blueprints.
*/
UCLASS(meta = (Transient, ScriptName = "ConstraintsScriptingLibrary"), MinimalAPI)
class UConstraintsScriptingLibrary : public UBlueprintFunctionLibrary
{

public:
	GENERATED_BODY()

public:

	/**
	* No longer used, the system will internally set up the outers for the handles and see UConstraintSubsystem
	* for the delegates.
	*/
	UE_DEPRECATED(5.3, "Please use UConstraintSubsystem delegates.")
	static CONSTRAINTS_API UConstraintsManager* GetManager(UWorld* InWorld);

	/**
	* Create the transformable handle that deals with getting and setting transforms on this scene component
	* @param InWorld, the world you are in
	* @param InSceneComponent World to create the constraint
	* @param InSocketName Optional name of the socket to get the transform 
	* @return returns the handle for this scene component
	**/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig | Constraints")
	static CONSTRAINTS_API UTransformableComponentHandle* CreateTransformableComponentHandle(UWorld* InWorld, USceneComponent* InSceneComponent, const FName& InSocketName);

	/**
	* Create the transformable handle that deals with getting and setting transforms on this object
	* @param InWorld, the world you are in
	* @param InObject World to create the constraint
	* @param InAttachmentName Optional name of the attachment to get the transform. Not that this can represent a scene component's socket name or a control rig control for example. 
	* @return returns the handle for this scene component
	**/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig | Constraints")
	static CONSTRAINTS_API UTransformableHandle* CreateTransformableHandle(UWorld* InWorld, UObject* InObject, const FName& InAttachmentName = NAME_None);

	/**
	* Create Constraint based on the specified type.
	* @param InWorld World to create the constraint
	* @param InType The type of constraint
	* @return return The constraint object
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig | Constraints")
	static CONSTRAINTS_API UTickableTransformConstraint* CreateFromType(UWorld* InWorld, const ETransformConstraintType InType);

	/**
	* Add Constraint to the system using the incoming parent and child handles with the specified type.
	* @param InWorld World to create the constraint
	* @param InParentHandle The parent handle
	* @param InChildHandle The child handle
	* @param InConsrtaint The constraint
	* @return return If constraint added correctly
	*/
	UE_DEPRECATED(5.4, "UConstraintsScriptingLibrary::AddConstraint is deprecated. use UControlRigSequencerEditorLibrary::AddConstraint instead.")
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig | Constraints")
	static CONSTRAINTS_API bool AddConstraint(UWorld* InWorld, UTransformableHandle* InParentHandle, UTransformableHandle* InChildHandle, UTickableTransformConstraint *InConstraint,
			const bool bMaintainOffset);


	/* Get a copy of the constraints in the current world
	@param InWorld World we are in
	@return Copy of the constraints in the level
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig | Constraints")
	static CONSTRAINTS_API TArray<UTickableConstraint*> GetConstraintsArray(UWorld* InWorld);


	/**
	* Remove specified constraint 
	* @param InWorld World to remove the constraint
	* @param InTickableConstraint Constraint to remove
	* @return return If constraint removed correctly
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig | Constraints")
	static CONSTRAINTS_API bool RemoveThisConstraint(UWorld* InWorld, UTickableConstraint* InTickableConstraint);

	/**
	* Remove constraint at specified index
	* @param InWorld World to remove the constraint
	* @param InIndex Index to remove from
	* @return return If constraint removed correctly
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig | Constraints")
	static CONSTRAINTS_API bool RemoveConstraint(UWorld* InWorld, int32 InIndex);

};
