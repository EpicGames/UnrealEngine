// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/SkeletalMeshComponent.h"
#include "Containers/Map.h"
#include "Delegates/IDelegateInstance.h"
#include "Math/Transform.h"
#include "UObject/WeakObjectPtrTemplates.h"

namespace UE::Anim
{

/**
 * FAnimationEvaluationTask allows post-evaluation functions to be added to an evaluator, so that data not cached in FAnimationEvaluationContext
 * can be handled by external mechanisms.
*/
	
struct FAnimationEvaluationTask
{
	bool IsValid(const USkeletalMeshComponent* InSkeletalMeshComponent) const
	{
		return Guid.IsValid() && PostEvaluationFunction.IsSet() && InSkeletalMeshComponent && InSkeletalMeshComponent == SkeletalMeshComponent;
	}

	FGuid Guid;
	TWeakObjectPtr<USkeletalMeshComponent> SkeletalMeshComponent = nullptr;
	TFunction<void()> PostEvaluationFunction;
};
	
/**
 * FAnimationEvaluator allows evaluating the animation of a skeletal mesh without having to tick it directly.
 * USkeletalMeshComponent::TickAnimation & USkeletalMeshComponent::RefreshBoneTransforms update internal data, in particular bones transforms,
 * which leads to several side effects (motion blur for example) as well as poor performance.
 * This structure provides a cached equivalent of USkeletalMeshComponent::RefreshBoneTransforms, which is only updated on demand.
 * It's also bound to USkeletalMeshComponent::OnBoneTransformsFinalizedMC so that it's updated with the real values once the skeletal mesh is 
 * finally up to date.
*/
	
struct FAnimationEvaluator
{
	FAnimationEvaluator(USkeletalMeshComponent* InSkeletalMeshComponent);
	~FAnimationEvaluator();

	/** Clears the context and refresh the bones transforms if bRefreshBoneTransforms is true. */ 
	void Update(const bool bRefreshBoneTransforms);
	
	/** Returns true if the skeletal mesh component, its skeletal mesh and the transforms are valid for use.
	 * If bForTransform is true, the sizes of the component space transforms from the skeletal mesh component and those of the context are tested. 
	 */
	bool IsValid(const bool bForTransform = false) const;
	
	/** Returns InSocketName component space transform composed with the skeletal mesh component's global transform. */
	CONSTRAINTS_API FTransform GetGlobalTransform(const FName InSocketName) const;

	/** Returns InSocketName component space transform composed with the skeletal mesh component's relative transform. */
	FTransform GetRelativeTransform(const FName InSocketName) const;

	/** Adds a post-evaluation task to this evaluator if it has not already been added (returns true if added). */
	bool AddPostEvaluationTask(const FAnimationEvaluationTask& InTask);
	
private:

	FAnimationEvaluator() = default;

	/** Bind/Unbind this to the SkeletalMeshComponent's OnBoneTransformsFinalizedMC delegate. */
	void BindDelegates();
	void UnbindDelegates();
	
	/** Prepares the animation context for evaluation. */
	void UpdateContext();
	
	/** Evaluates the animation using the Context. */
	void EvaluateAnimation();

	/** Evaluates the animation and store the bones transforms. */
	void RefreshBoneTransforms();

	/** Updates the bones transforms once the skeletal mesh is finally up to date. */
	void BoneTransformsFinalized();
	
	TWeakObjectPtr<USkeletalMeshComponent> SkeletalMeshComponent = nullptr;
	FAnimationEvaluationContext Context;
	FDelegateHandle OnBoneTransformsFinalizedHandle;
	TMap<FGuid, FAnimationEvaluationTask> PostEvaluationTasks;
};

/**
 * FAnimationEvaluationCache provides a cache mechanism connecting skeletal mesh component and its animation evaluator to avoid multiplying
 * FAnimationEvaluator instances when they all refer to the same skeletal mesh component. The animation evaluator is built lazily and dirtied
 * manually on demand  that the next call to GetEvaluator actually evaluates the animation if needed.  
*/
	
struct FAnimationEvaluationCache
{
public:
	static FAnimationEvaluationCache& Get();
	~FAnimationEvaluationCache();

	/** Marks the evaluator related to InSkeletalMeshComponent for evaluation. */
	void MarkForEvaluation(const USkeletalMeshComponent* InSkeletalMeshComponent);
	
	/**
	 * Returns and up-to-date evaluator to be queried.
	 * Note that calling this function will actually evaluate the animation if the evaluator has been previously dirtied.
	 */
	const FAnimationEvaluator& GetEvaluator(USkeletalMeshComponent* InSkeletalMeshComponent);
	
	/**
	 * Returns and up-to-date evaluator to be queried and adds the post-evaluation task if it has not already been added.
	 * Note that calling this function will actually evaluate the animation and the post-evaluation tasks if the evaluator has been previously dirtied.
	 */
	const FAnimationEvaluator& GetEvaluator(USkeletalMeshComponent* InSkeletalMeshComponent, const FAnimationEvaluationTask& InTask);
	
	/**
	 * Returns the evaluator related to InSkeletalMeshComponent if any or an invalid one otherwise.
	 * Note that calling this function won't update the evaluator so make sure to use FAnimationEvaluator::IsValid after calling that function.  
	 */
	const FAnimationEvaluator& FindEvaluator(const USkeletalMeshComponent* InSkeletalMeshComponent) const;

private:
	FAnimationEvaluationCache() = default;

	/** Listen to the constraint manager notification to avoid keeping useless or invalid data. */
	void RegisterNotifications();

	/** Stop listening to the constraint manager notification system. */
	void UnregisterNotifications();

	/** List of connected skeletal mesh components and animation evaluators. */
	TMap<USkeletalMeshComponent*, TUniquePtr<FAnimationEvaluator>> PerSkeletalMeshEvaluator;

	FDelegateHandle ConstraintsNotificationHandle;
};

/**
 * FEvaluationForCachingScope provides a way to set the delta time when updating the animations.
 * This class is currently not thread safe and won't update the delta time on re-entrant calls.
*/

struct FEvaluationForCachingScope : public FNoncopyable
{
	CONSTRAINTS_API FEvaluationForCachingScope(const float InDeltaTime = 1.f / 30.f);
	CONSTRAINTS_API ~FEvaluationForCachingScope();
};

CONSTRAINTS_API bool IsCaching();
	
}