// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

struct FTickFunction;
class UTickableConstraint;
class UTransformableHandle;
class UTickableTransformConstraint;
class UWorld;

#define UE_API CONSTRAINTS_API

/** 
 * FDependencyBuilder
 **/

struct FDependencyBuilder
{
public: 
	/** Ensure default dependencies between constraints. */
	static bool BuildDependencies(UWorld* InWorld, UTickableTransformConstraint* Constraint);

	/** Returns true if dependencies should be logged. */
	static bool LogDependencies();
	
private:
	
	/** Ensures that attachment dependencies are reflected at the constraints level. */
	static void BuildAttachmentsDependencies(UWorld* InWorld, const UTickableTransformConstraint* InConstraint);

	/** Ensures that internal dependencies (control rig only at this point) are addressed at the constraints level. */
	static void BuildSelfDependencies(UWorld* InWorld, UTickableTransformConstraint* InConstraint);

	/** Ensures that external dependencies are addressed at the constraints level. */
	static void BuildExternalDependencies(UWorld* InWorld, UTickableTransformConstraint* InConstraint);
};

/**
 * FConstraintDependencyScope provides a way to build constraint dependencies when the constraint is not valid when added to the subsystem
 * but after (when resolving sequencer or control rig bindings).
 * The dependencies will be built on destruction if the constraint's validity changed within the lifetime of that object.
 */

struct FConstraintDependencyScope
{
	UE_API FConstraintDependencyScope(UTickableTransformConstraint* InConstraint, UWorld* InWorld = nullptr);
	UE_API ~FConstraintDependencyScope();
private:
	TWeakObjectPtr<UTickableTransformConstraint> WeakConstraint = nullptr;
	TWeakObjectPtr<UWorld> WeakWorld = nullptr;
	bool bPreviousValidity = false;
};

/**
 * FHandleDependencyChecker provides a way to check (direct + constraints + tick) dependencies between two UTransformableHandle
 * HasDependency will return true if InHandle depends on InParentToCheck.
 */

struct FHandleDependencyChecker
{
	FHandleDependencyChecker(UWorld* InWorld = nullptr);
	bool HasDependency(const UTransformableHandle& InHandle, const UTransformableHandle& InParentToCheck) const;		
private:
	TWeakObjectPtr<UWorld> WeakWorld = nullptr;
};

/** 
 * FConstraintCycleChecker
 **/

struct FConstraintCycleChecker
{
public:
	/** Checks if this handle is cycle from a tick dependencies perspective. */
	static bool IsCycling(const TWeakObjectPtr<UTransformableHandle>& InHandle);

	/** Checks for cycling constraints and manage tick dependencies if needed to avoid cycles from a tick dependency pov. */
	static void CheckAndFixCycles(const UTickableTransformConstraint* InConstraint);
	
	// ensure that InPossiblePrimary is not depending on InPossibleSecondary to avoid creating cycles 
	static bool HasPrerequisiteDependencyWith(const FTickFunction* InSecondary, const FTickFunction* InPrimary, TSet<const FTickFunction*>& InOutVisitedFunctions);

private:
	/**
	 * Manage tick dependencies if needed to avoid cycles from a tick dependency pov.
	 * Both InConstraintToUpdate and its parent handle are supposed valid at this point
	 */
	static void UpdateCyclingDependency(UWorld* InWorld, UTickableTransformConstraint* InConstraintToUpdate);
	
	using ConstraintPtr = TWeakObjectPtr<UTickableConstraint>;
	using ConstraintArray = TArray<ConstraintPtr>;
};

#undef UE_API