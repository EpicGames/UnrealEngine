// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Constraint.h"
#include "ConstraintsManager.h"

#include "Kismet/BlueprintFunctionLibrary.h"

#include "Delegates/DelegateCombinations.h"
#include "Delegates/Delegate.h"

#include "TransformConstraint.generated.h"

enum class EHandleEvent : uint8;
class UTransformableHandle;
class UTransformableComponentHandle;
class USceneComponent;
enum class EMovieSceneTransformChannel : uint32;
class UWorld;
/** 
 * UTickableTransformConstraint
 **/

UCLASS(Abstract, Blueprintable, MinimalAPI)
class UTickableTransformConstraint : public UTickableConstraint
{
	GENERATED_BODY()

public:

	/** Sets up the constraint so that the initial offset is set and dependencies and handles managed. */
	CONSTRAINTS_API void Setup();
	
	/** UObjects overrides. */
	CONSTRAINTS_API virtual void PostLoad() override;
	CONSTRAINTS_API virtual void PostDuplicate(bool bDuplicateForPIE) override;

	/** Returns the target hash value (i.e. the child handle's hash). */
	CONSTRAINTS_API virtual uint32 GetTargetHash() const override;
	
	/** Test whether an InObject is referenced by that constraint. (i.e. is it's parent or child). */
	CONSTRAINTS_API virtual bool ReferencesObject(TWeakObjectPtr<UObject> InObject) const override;
	/** If true it contains objects bound to an external system, like sequencer so we don't do certain things, like remove constraints when they don't resolve*/
	CONSTRAINTS_API virtual bool HasBoundObjects() const override;
	
	/** Resolve the bound objects so that any object it references are resovled and correctly set up*/
	CONSTRAINTS_API virtual void ResolveBoundObjects(FMovieSceneSequenceID LocalSequenceID, TSharedRef<UE::MovieScene::FSharedPlaybackState> SharedPlaybackState, UObject* SubObject = nullptr) override;

	/** Whether or not it's valid for example it may not be fully loaded*/
	virtual bool IsValid(const bool bDeepCheck = true) const override;

	/** If Active and the handles and targets are valid, and tick function is registered*/
	CONSTRAINTS_API virtual bool IsFullyActive() const override;

	/** If that constraint needs to be handled by the compensation system. */
	CONSTRAINTS_API virtual bool NeedsCompensation() const;

	/** Create duplicate with new Outer*/
	CONSTRAINTS_API virtual UTickableConstraint* Duplicate(UObject* NewOuter) const override;

	/** Override the evaluate so we can tick our handles*/
	CONSTRAINTS_API virtual void Evaluate(bool bTickHandlesAlso = false) const override;

	/** Sets the Active value and enable/disable the tick function. */
	CONSTRAINTS_API virtual void SetActive(const bool bIsActive) override;
	
	/** The transformable handle representing the parent of that constraint. */
	UPROPERTY(BlueprintReadWrite, Category = "Handle")
	TObjectPtr<UTransformableHandle> ParentTRSHandle;
	
	/** The transformable handle representing the child of that constraint. */
	UPROPERTY(BlueprintReadWrite, Category = "Handle")
	TObjectPtr<UTransformableHandle> ChildTRSHandle;

	/** Should that constraint maintain the default offset.  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Offset", meta=(EditCondition="Type!=ETransformConstraintType::LookAt"))
	bool bMaintainOffset = true;

	/** Defines how much the constraint will be applied. */
	// UPROPERTY(EditAnywhere, BlueprintReadWrite,Category="Weight", meta = (Input, ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	//@benoit when not EditAnywhere?
	UPROPERTY(meta = (Input, ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float Weight = 1.f;

	/** Should the child be able to change it's offset dynamically. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Offset", meta=(EditCondition="(Type!=ETransformConstraintType::LookAt) && bMaintainOffset"))
	bool bDynamicOffset = false;

	/** Returns the constraint type (Position, Parent, Aim...). */
	CONSTRAINTS_API int64 GetType() const;

	/** Get the current child's global transform. */
	CONSTRAINTS_API FTransform GetChildGlobalTransform() const;
	
	/** Get the current child's local transform. */
	CONSTRAINTS_API FTransform GetChildLocalTransform() const;

	/** Get the current parent's global transform. */
	CONSTRAINTS_API FTransform GetParentGlobalTransform() const;
	
	/** Get the current parent's local transform. */
	CONSTRAINTS_API FTransform GetParentLocalTransform() const;

	/** Returns the channels to key based on the constraint's type. */
	CONSTRAINTS_API EMovieSceneTransformChannel GetChannelsToKey() const;

	/**
	 * Manages changes on the child/parent transformable handle. This can be used to update internal data (i.e. offset)
	 * when transform changes outside of that system and need to trigger changes within the constraint itself.   
	*/
	CONSTRAINTS_API virtual void OnHandleModified(UTransformableHandle* InHandle, EHandleEvent InEvent);

	/** Returns the handles tick function (ensuring it lives in the same world). */
	CONSTRAINTS_API FTickFunction* GetChildHandleTickFunction() const;
	CONSTRAINTS_API FTickFunction* GetParentHandleTickFunction() const;

	/**
	* Sets up dependencies with the first primary prerequisite available if the parent does not tick.   
	*/
	CONSTRAINTS_API void EnsurePrimaryDependency(const UWorld* InWorld);
	
protected:

	/** Registers/Unregisters useful delegates for both child and parent handles. */
	CONSTRAINTS_API void UnregisterDelegates() const;
	CONSTRAINTS_API void RegisterDelegates();

	
	/**
	 * Computes the initial offset that is needed to keep the child's global transform unchanged when creating the
	 * constraint. This can be called whenever necessary to update the current offset.
	*/
	virtual void ComputeOffset() PURE_VIRTUAL(ComputeOffset, return;);
	
	/**
	 * Sets up dependencies between the parent, the constraint and the child using their respective tick functions.
	 * It creates a dependency graph between them so that they tick in the right order when evaluated.   
	*/
	CONSTRAINTS_API void SetupDependencies(const UWorld* InWorld);

	/** Set the current child's global transform. */
	CONSTRAINTS_API void SetChildGlobalTransform(const FTransform& InGlobal) const;
	
	/** Set the current child's local transform. */
	CONSTRAINTS_API void SetChildLocalTransform(const FTransform& InLocal) const;

	/** Defines the constraint's type (Position, Parent, Aim...). */
	UPROPERTY()
	ETransformConstraintType Type = ETransformConstraintType::Parent;

	/** Handle active state modification if needed. */
	UE_DEPRECATED(5.8, "Deprecated, no longer used.")
	void OnActiveStateChanged() const {};

	/** Returns the handle's tick function (ensuring it lives in the same world). */
	CONSTRAINTS_API FTickFunction* GetHandleTickFunction(const TObjectPtr<UTransformableHandle>& InHandle) const;

	/** Perform necessary pre-evaluation steps before calculating the constraint's result. */
	void PreEvaluate() const;

	/** Perform necessary post-evaluation steps after calculating the constraint's result. */
    void PostEvaluate() const;

public:
	/** (Re-)Registers the constraint function and (re-)binds the required delegates*/
	CONSTRAINTS_API virtual void InitConstraint(UWorld * InWorld) override;
	CONSTRAINTS_API virtual void TeardownConstraint(UWorld * InWorld) override;
	CONSTRAINTS_API virtual void AddedToWorld(UWorld* InWorld) override;

	/** Used at creation to define whether the current relative offset should be used instead of the provided one. */
	UPROPERTY(EditAnywhere, Transient, Category="Offset", meta = (DisplayPriority=0))
	bool bUseCurrentOffset = true;
	
#if WITH_EDITOR
public:
	/** Returns the constraint's label used for UI. */
	CONSTRAINTS_API virtual FString GetLabel() const override;
	CONSTRAINTS_API virtual FString GetFullLabel() const override;

	/** Returns the constraint's type label used for UI. */
	CONSTRAINTS_API virtual FString GetTypeLabel() const override;

	// UObject interface
	CONSTRAINTS_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	CONSTRAINTS_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	CONSTRAINTS_API virtual void PostEditUndo() override;
	// End of UObject interface

	/** Returns a delegate that can be used to monitor for property changes. This can be used to monitor constraints changes
	 * only instead of using FCoreUObjectDelegates::OnObjectPropertyChanged that is listening to every objects.
	 * Note that bScaling is currently the only property change we monitor but this can be used for other properties.
	 */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnConstraintChanged, UTickableTransformConstraint*, const FPropertyChangedEvent&);
	static CONSTRAINTS_API FOnConstraintChanged& GetOnConstraintChanged();

protected:
	static CONSTRAINTS_API FOnConstraintChanged OnConstraintChanged;
#endif
};

/** 
 * UTickableTranslationConstraint
 **/

UCLASS(Blueprintable, MinimalAPI)
class UTickableTranslationConstraint : public UTickableTransformConstraint
{
	GENERATED_BODY()

public:
	CONSTRAINTS_API UTickableTranslationConstraint();

	/** Returns the position constraint function that the tick function will evaluate. */
	CONSTRAINTS_API virtual FConstraintTickFunction::ConstraintFunction GetFunction() const override;

	/** Updates the dynamic offset based on external child's transform changes. */
	CONSTRAINTS_API virtual void OnHandleModified(UTransformableHandle* InHandle, EHandleEvent InEvent) override;

protected:

	/** Cache data structure to store last child local/global location. */
	struct FDynamicCache
	{
		uint32 CachedInputHash = 0;
	};
	mutable FDynamicCache Cache;
	CONSTRAINTS_API uint32 CalculateInputHash() const;
	
	/** Computes the child's local translation offset in the parent space. */
	CONSTRAINTS_API virtual void ComputeOffset() override;

public:
	/** Defines the local child's translation offset in the parent space. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Offset", meta=(EditCondition="bMaintainOffset"))
	FVector OffsetTranslation = FVector::ZeroVector;

	/** Defines which translation axis is constrained. */
	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = "Axis Filter")
	FFilterOptionPerAxis AxisFilter;

#if WITH_EDITOR
public:
	// UObject interface
	CONSTRAINTS_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	// End of UObject interface
#endif
};

/** 
 * UTickableRotationConstraint
 **/

UCLASS(Blueprintable, MinimalAPI)
class UTickableRotationConstraint : public UTickableTransformConstraint
{
	GENERATED_BODY()

public:
	CONSTRAINTS_API UTickableRotationConstraint();

	/** Returns the rotation constraint function that the tick function will evaluate. */
	CONSTRAINTS_API virtual FConstraintTickFunction::ConstraintFunction GetFunction() const override;

	/** Updates the dynamic offset based on external child's transform changes. */
	CONSTRAINTS_API virtual void OnHandleModified(UTransformableHandle* InHandle, EHandleEvent InEvent) override;

protected:

	/** Cache data structure to store last child local/global rotation. */
	struct FDynamicCache
	{
		uint32 CachedInputHash = 0;
	};
	mutable FDynamicCache Cache;
	CONSTRAINTS_API uint32 CalculateInputHash() const;
	
	/** Computes the child's local rotation offset in the parent space. */
	CONSTRAINTS_API virtual void ComputeOffset() override;

public:
	/** Defines the local child's rotation offset in the parent space. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Offset", meta=(EditCondition="bMaintainOffset"))
	FQuat OffsetRotation = FQuat::Identity;

	/** Defines which rotation axis is constrained. */
	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = "Axis Filter")
	FFilterOptionPerAxis AxisFilter;

#if WITH_EDITOR
public:
	// UObject interface
	CONSTRAINTS_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	// End of UObject interface
#endif
};

/** 
 * UTickableScaleConstraint
 **/

UCLASS(Blueprintable, MinimalAPI)
class UTickableScaleConstraint : public UTickableTransformConstraint
{
	GENERATED_BODY()

public:
	CONSTRAINTS_API UTickableScaleConstraint();
	
	/** Returns the scale constraint function that the tick function will evaluate. */
	CONSTRAINTS_API virtual FConstraintTickFunction::ConstraintFunction GetFunction() const override;

	/** Updates the dynamic offset based on external child's transform changes. */
	CONSTRAINTS_API virtual void OnHandleModified(UTransformableHandle* InHandle, EHandleEvent InEvent) override;
	
protected:
	/** Computes the child's local scale offset in the parent space. */
	CONSTRAINTS_API virtual void ComputeOffset() override;

	/** Cache data structure to store last child local/global transform. */
	struct FDynamicCache
	{
		uint32 CachedInputHash = 0;
	};
	mutable FDynamicCache Cache;
	CONSTRAINTS_API uint32 CalculateInputHash() const;

public:
	/** Defines the local child's scale offset in the parent space. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Offset", meta=(EditCondition="bMaintainOffset"))
	FVector OffsetScale = FVector::OneVector;

	/** Defines which scale axis is constrained. */
	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = "Axis Filter")
	FFilterOptionPerAxis AxisFilter;

#if WITH_EDITOR
public:
	// UObject interface
	CONSTRAINTS_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	// End of UObject interface
#endif
};

/** 
 * UTickableParentConstraint
 **/

UCLASS(Blueprintable, MinimalAPI)
class UTickableParentConstraint : public UTickableTransformConstraint
{
	GENERATED_BODY()

public:
	CONSTRAINTS_API UTickableParentConstraint();
	
	/** Returns the transform constraint function that the tick function will evaluate. */
	CONSTRAINTS_API virtual FConstraintTickFunction::ConstraintFunction GetFunction() const override;

	bool IsScalingEnabled() const
	{
		return bScaling;
	}
	
	void SetScaling(const bool bInScale)
	{
		bScaling = bInScale;
	}

	static FName GetScalingPropertyName() { return GET_MEMBER_NAME_CHECKED(UTickableParentConstraint, bScaling); }

	/** Updates the dynamic offset based on external child's transform changes. */
	CONSTRAINTS_API virtual void OnHandleModified(UTransformableHandle* InHandle, EHandleEvent InEvent) override;

protected:
	/** Cache data structure to store last child local/global transform. */
	struct FDynamicCache
	{
		uint32 CachedInputHash = 0;
	};
	mutable FDynamicCache Cache;
	CONSTRAINTS_API uint32 CalculateInputHash() const;

	/** Computes the child's local transform offset in the parent space. */
	CONSTRAINTS_API virtual void ComputeOffset() override;

public:
	/** Defines the local child's transform offset in the parent space. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Offset", meta=(EditCondition="bMaintainOffset"))
	FTransform OffsetTransform = FTransform::Identity;

	/** Defines whether we propagate the parent scale. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Properties")
	bool bScaling = false;

	/** Defines which translation/rotation/scale axis are constrained. */
	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = "Axis Filter") 
	FTransformFilter TransformFilter;

#if WITH_EDITOR
public:
	// UObject interface
	CONSTRAINTS_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	// End of UObject interface
#endif
};

/** 
 * UTickableLookAtConstraint
 **/

UCLASS(Blueprintable, MinimalAPI)
class UTickableLookAtConstraint : public UTickableTransformConstraint
{
	GENERATED_BODY()

public:
	CONSTRAINTS_API UTickableLookAtConstraint();
	
	/** Returns the look at constraint function that the tick function will evaluate. */
	CONSTRAINTS_API virtual FConstraintTickFunction::ConstraintFunction GetFunction() const override;

	/** If that constraint needs to be handled by the compensation system. */
	CONSTRAINTS_API virtual bool NeedsCompensation() const override;

protected:
	/**
     * Computes the initial axis that is needed to keep the child's orientation unchanged when creating the constraint.
    */
	CONSTRAINTS_API virtual void ComputeOffset() override;

	/** Defines the aiming axis. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Axis")
	FVector Axis = FVector::XAxisVector;

private:

	/** Computes the shortest quaternion between A and B. */
	static FQuat FindQuatBetweenNormals(const FVector& A, const FVector& B);

#if WITH_EDITOR
public:
	// UObject interface
	CONSTRAINTS_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	// End of UObject interface
#endif
};

struct FTransformConstraintUtils
{
	UE_DEPRECATED(5.6, "This function was moved to the UE::TransformConstraintUtil namespace.")
	static CONSTRAINTS_API void GetParentConstraints(
		UWorld* World,
		const AActor* InChild,
		TArray< TWeakObjectPtr<UTickableConstraint> >& OutConstraints);

	UE_DEPRECATED(5.6, "This function was moved to the UE::TransformConstraintUtil namespace.")
	static CONSTRAINTS_API UTransformableComponentHandle* CreateHandleForSceneComponent(
		USceneComponent* InSceneComponent,
		const FName& InSocketName);

	UE_DEPRECATED(5.6, "This function was moved to the UE::TransformConstraintUtil namespace.")
	static CONSTRAINTS_API UTickableTransformConstraint* CreateFromType(
		UWorld* InWorld,
		const ETransformConstraintType InType,
		const bool bUseDefault = false);

	UE_DEPRECATED(5.6, "This function was moved to the UE::TransformConstraintUtil namespace.")	
	static CONSTRAINTS_API UTickableTransformConstraint* CreateAndAddFromObjects(
		UWorld* InWorld,
		UObject* InParent, const FName& InParentSocketName,
		UObject* InChild, const FName& InChildSocketName,
		const ETransformConstraintType InType,
		const bool bMaintainOffset = true,
		const bool bUseDefault = false);

	UE_DEPRECATED(5.6, "This function was moved to the UE::TransformConstraintUtil namespace.")	
	static CONSTRAINTS_API bool AddConstraint(
		UWorld* InWorld,
		UTransformableHandle* InParentHandle,
		UTransformableHandle* InChildHandle,
		UTickableTransformConstraint* Constraint,
		const bool bMaintainOffset = true,
		const bool bUseDefault = false);

	UE_DEPRECATED(5.6, "This function was moved to the UE::TransformConstraintUtil namespace.")
	static CONSTRAINTS_API FTransform ComputeRelativeTransform(
		const FTransform& InChildLocal,
		const FTransform& InChildWorld,
		const FTransform& InSpaceWorld,
		const UTickableTransformConstraint* InConstraint);

	UE_DEPRECATED(5.6, "This function was moved to the UE::TransformConstraintUtil namespace.")
	static CONSTRAINTS_API TOptional<FTransform> GetRelativeTransform(UWorld* InWorld, const uint32 InHandleHash);
	static CONSTRAINTS_API TOptional<FTransform> GetConstraintsRelativeTransform(
		const TArray< TWeakObjectPtr<UTickableConstraint> >& InConstraints,
		const FTransform& InChildLocal, const FTransform& InChildWorld);

	UE_DEPRECATED(5.6, "This function was moved to the UE::TransformConstraintUtil namespace.")
	static CONSTRAINTS_API int32 GetLastActiveConstraintIndex(const TArray< TWeakObjectPtr<UTickableConstraint> >& InConstraints);

	UE_DEPRECATED(5.6, "This function was moved to the UE::TransformConstraintUtil namespace.")
	static CONSTRAINTS_API void GetChildrenConstraints(
		UWorld* World,
		const UTickableTransformConstraint* InConstraint,
		TArray< TWeakObjectPtr<UTickableConstraint> >& OutConstraints,
		const bool bIncludeTarget = false);
	
	UE_DEPRECATED(5.6, "This function was moved to the UE::TransformConstraintUtil namespace.")
	static CONSTRAINTS_API void UpdateTransformBasedOnConstraint(FTransform& CurrentTransform, USceneComponent* SceneComponent);
};
