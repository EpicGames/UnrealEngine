// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Constraint.h"

#include "Containers/Array.h"
#include "Containers/Map.h"

#include "Delegates/IDelegateInstance.h"
#include "UObject/ObjectKey.h"

#define UE_API CONSTRAINTS_API

class UWorld;
class UTickableConstraint;
class UTransformableHandle;
class UTickableTransformConstraint;
class UTransformableComponentHandle;

namespace UE::TransformConstraintUtil
{

/**
 * FConstraintsInteractionCache is designed to minimize the number of requests made to the ConstraintManager and thus improve performance.
 * it is particularly useful for interface components that need to be updated frequently.
 * note that PerHandleConstraints and PerTargetConstraints are reset when the constraint graph is updated, to keep it synchronized.
 */
	
struct FConstraintsInteractionCache
{
public:
	UE_API ~FConstraintsInteractionCache();
	
	UE_API void Reset();

	UE_API const TArray< TWeakObjectPtr<UTickableConstraint> >& Get(const UObject* InObject, const FName& InAttachmentName);
	UE_API bool HasAnyActiveConstraint(const UObject* InObject, const FName& InAttachmentName);
	UE_API TOptional<FTransform> GetParentTransform(const UObject* InObject, const FName& InAttachmentName);

	UE_API const TArray< TWeakObjectPtr<UTickableConstraint> >& Get(const uint32 InHandleHash, UWorld* InWorld);
	UE_API bool HasAnyActiveConstraint(const uint32 InHandleHash, UWorld* InWorld);
	UE_API TOptional<FTransform> GetParentTransform(const uint32 InHandleHash, UWorld* InWorld);

	UE_API const TArray< TWeakObjectPtr<UTickableConstraint> >& Get(UObject* InTarget, UWorld* InWorld = nullptr);
	UE_API bool HasAnyActiveConstraint(UObject* InTarget, UWorld* InWorld = nullptr);

	UE_API bool HasAnyDependency(UObject* InChild, UObject* InParent, UWorld* InWorld);
	
	UE_API void RegisterNotifications();
	UE_API void UnregisterNotifications();

private:

	bool HasAnyDependencyInternal(UObject* InChild, UObject* InParent, UWorld* InWorld, TSet< UObject* >& Visited);
	
	TMap<uint32, TArray< TWeakObjectPtr<UTickableConstraint> >> PerHandleConstraints;
	TMap<TObjectKey<UObject>, TArray< TWeakObjectPtr<UTickableConstraint> >> PerTargetConstraints;
	FDelegateHandle ConstraintsNotificationHandle;
};

}


namespace UE::TransformConstraintUtil
{

	/** Fills a sorted constraint array that InChild actor is the child of. */
	CONSTRAINTS_API void GetParentConstraints(
		UWorld* World,
		const AActor* InChild,
		TArray< TWeakObjectPtr<UTickableConstraint> >& OutConstraints);

	/** Create a handle for the scene component.*/
	CONSTRAINTS_API UTransformableComponentHandle* CreateHandleForSceneComponent(
		USceneComponent* InSceneComponent,
		const FName& InSocketName);

	/** Creates a new transform constraint based on the InType. */
	CONSTRAINTS_API UTickableTransformConstraint* CreateFromType(
		UWorld* InWorld,
		const ETransformConstraintType InType,
		const bool bUseDefault = false);

	/** Creates respective handles and creates a new InType transform constraint. */	
	CONSTRAINTS_API UTickableTransformConstraint* CreateAndAddFromObjects(
		UWorld* InWorld,
		UObject* InParent, const FName& InParentSocketName,
		UObject* InChild, const FName& InChildSocketName,
		const ETransformConstraintType InType,
		const bool bMaintainOffset = true,
		const bool bUseDefault = false,
		const TFunction<void()>& InValidDependencyFunction = nullptr);

	/** Registers a new transform constraint within the constraints manager. */	
	CONSTRAINTS_API bool AddConstraint(
		UWorld* InWorld,
		UTransformableHandle* InParentHandle,
		UTransformableHandle* InChildHandle,
		UTickableTransformConstraint* Constraint,
		const bool bMaintainOffset = true,
		const bool bUseDefault = false);

	/** Computes the relative transform between both transform based on the constraint's InType. */
	CONSTRAINTS_API FTransform ComputeRelativeTransform(
		const FTransform& InChildLocal,
		const FTransform& InChildWorld,
		const FTransform& InSpaceWorld,
		const UTickableTransformConstraint* InConstraint);

	/** Computes the current constraint space local transform. */
	CONSTRAINTS_API TOptional<FTransform> GetRelativeTransform(UWorld* InWorld, const uint32 InHandleHash);
	CONSTRAINTS_API TOptional<FTransform> GetConstraintsRelativeTransform(
		const TArray< TWeakObjectPtr<UTickableConstraint> >& InConstraints,
		const FTransform& InChildLocal, const FTransform& InChildWorld);

	/** Get the last active constraint that has dynamic offset. */
	CONSTRAINTS_API int32 GetLastActiveConstraintIndex(const TArray< TWeakObjectPtr<UTickableConstraint> >& InConstraints);

	/**
	 * Fills a constraint array that InConstraint->ChildHandle is the parent of.
	 * If bIncludeTarget is true, we also get the other constraints that act on the same target.
	 */
	CONSTRAINTS_API void GetChildrenConstraints(
		UWorld* World,
		const UTickableTransformConstraint* InConstraint,
		TArray< TWeakObjectPtr<UTickableConstraint> >& OutConstraints,
		const bool bIncludeTarget = false);
	
	/** Adjust the transform on a scene component so it's effected by the constraint*/
	CONSTRAINTS_API void UpdateTransformBasedOnConstraint(FTransform& InOutCurrentTransform, const USceneComponent* InSceneComponent);

	/** Returns the configuration constraint instance for a specific constraint class if it exists */
	CONSTRAINTS_API UTickableTransformConstraint* GetConfig(const UClass* InConstraintClass);
}

#undef UE_API
