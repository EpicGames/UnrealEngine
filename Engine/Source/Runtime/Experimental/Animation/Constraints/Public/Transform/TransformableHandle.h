// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "Engine/EngineBaseTypes.h"
#include "MovieSceneObjectBindingID.h"
#include "TransformableHandle.generated.h"

class UMovieSceneSection;
class USceneComponent;
enum class EMovieSceneTransformChannel : uint32;
struct FFRameNumber;
struct FFrameRate;
struct FMovieSceneDoubleChannel;
struct FMovieSceneFloatChannel;
struct FMovieSceneSequenceHierarchy;
struct FMovieSceneSequenceID;
struct FTickFunction;

namespace UE::MovieScene
{
	struct FFixedObjectBindingID;
	struct FSharedPlaybackState;
}

UENUM()
enum class EHandleEvent : uint8
{
	LocalTransformUpdated,
	GlobalTransformUpdated,
	ComponentUpdated,
	UpperDependencyUpdated,

	/** MAX - invalid */
	Max UMETA(Hidden),
};

/**
 * UTransformableHandle
 */

UCLASS(Abstract, Blueprintable, MinimalAPI)
class UTransformableHandle : public UObject 
{
	GENERATED_BODY()
	
public:

	DECLARE_EVENT_TwoParams(UTransformableHandle, FHandleModifiedEvent, UTransformableHandle*, EHandleEvent);
	
	CONSTRAINTS_API virtual ~UTransformableHandle();
	
	/** Sanity check to ensure the handle is safe to use.
	 * @param bDeepCheck to check that the transformable object it wraps is valid AND can be transformed. Default is true.
	 * Some handles, such as control handles, will only be bound to a skeletal mesh once the level sequence has been opened, but their control rig
	 * pointer and control name are valid, so they must be fully loaded so that the constraint can be updated later. bDeepCheck = false will be used
	 * for that purpose.
	 */
	virtual bool IsValid(const bool bDeepCheck = true) const PURE_VIRTUAL(IsValid, return false;);
	
	/** Sets the global transform of the underlying transformable object. */
	virtual void SetGlobalTransform(const FTransform& InGlobal) const PURE_VIRTUAL(SetGlobalTransform, );
	/** Sets the local transform of the underlying transformable object in it's parent space. */
	virtual void SetLocalTransform(const FTransform& InLocal) const PURE_VIRTUAL(SetLocalTransform, );
	/** Gets the global transform of the underlying transformable object. */
	virtual FTransform GetGlobalTransform() const PURE_VIRTUAL(GetGlobalTransform, return FTransform::Identity;);
	/** Gets the local transform of the underlying transformable object in it's parent space. */
	virtual FTransform GetLocalTransform() const PURE_VIRTUAL(GetLocalTransform, return FTransform::Identity;);

	/** If true it contains objects bound to an external system, like sequencer so we don't do certain things, like remove constraints when they don't resolve*/
	CONSTRAINTS_API virtual bool HasBoundObjects() const;

	/** Resolve the bound objects so that any object it references are resolved and correctly set up*/
	virtual void ResolveBoundObjects(FMovieSceneSequenceID LocalSequenceID, TSharedRef<UE::MovieScene::FSharedPlaybackState> SharedPlaybackState, UObject* SubObject = nullptr) PURE_VIRTUAL(ResolveBoundObjects);

	/** Make a duplicate of myself with this outer*/
	virtual UTransformableHandle* Duplicate(UObject* NewOuter) const PURE_VIRTUAL(Duplicate, return nullptr;);


	/** Fix up Binding in case it has changed*/
	CONSTRAINTS_API void OnBindingIDsUpdated(const TMap<UE::MovieScene::FFixedObjectBindingID, UE::MovieScene::FFixedObjectBindingID>& OldFixedToNewFixedMap, FMovieSceneSequenceID LocalSequenceID, TSharedRef<UE::MovieScene::FSharedPlaybackState> SharedPlaybackState);
	
	/** Perform any special ticking needed for this handle, by default it does nothing, todo need to see if we need to tick control rig also*/
	virtual void TickTarget() const {};

	/**
	 * Perform any pre-evaluation of the handle to ensure that the transform data are up to date.
	 * @param bTick to force any pre-evaluation ticking. Default is false.
	*/
	virtual void PreEvaluate(const bool bTick = false) const;
	
	/** Get the array of float channels for the specified section*/
	virtual TArrayView<FMovieSceneFloatChannel*>  GetFloatChannels(const UMovieSceneSection* InSection) const PURE_VIRTUAL(GetFloatChannels, return TArrayView<FMovieSceneFloatChannel*>(); );
	/** Get the array of double channels for the specified section*/
	virtual TArrayView<FMovieSceneDoubleChannel*>  GetDoubleChannels(const UMovieSceneSection * InSection) const PURE_VIRTUAL(GetDoubleChannels, return TArrayView<FMovieSceneDoubleChannel*>(); );
	/** Add Transform Keys at the specified times*/
	virtual bool AddTransformKeys(const TArray<FFrameNumber>& InFrames,
		const TArray<FTransform>& InTransforms,
		const EMovieSceneTransformChannel& InChannels,
		const FFrameRate& InTickResolution,
		UMovieSceneSection* InSection,
		const bool bLocal = true) const PURE_VIRTUAL(AddTransformKeys, return false;);

	/**
	 * Returns the target object containing the tick function (returned in GetTickFunction).
	 * See FTickFunction::AddPrerequisite for details.
	 **/
	virtual UObject* GetPrerequisiteObject() const PURE_VIRTUAL(GetPrerequisiteObject, return nullptr;);
	/**
	 * Returns the tick function of the underlying transformable object.
	 * This is used to set dependencies with the constraint.
	**/
	virtual FTickFunction* GetTickFunction() const PURE_VIRTUAL(GetTickFunction, return nullptr;);

	/** Generates a hash value of the underlying transformable object. */
	virtual uint32 GetHash() const PURE_VIRTUAL(GetHash, return 0;);

	/** Returns the underlying targeted object. */
	virtual TWeakObjectPtr<UObject> GetTarget() const PURE_VIRTUAL(GetTarget, return nullptr;);

	/** Check for direct dependencies with InOther. */
	virtual bool HasDirectDependencyWith(const UTransformableHandle& InOther) const PURE_VIRTUAL(HasDirectDependencyWith, return false;);
	
	/** Check for indirect dependencies with InOther. (returns false by default, override in child classes if needed) */
	virtual bool HasIndirectDependencyWith(const UTransformableHandle& InOther) const {return false;}
	
	/** Look for a possible tick function that can be used as a prerequisite. */
	virtual FTickPrerequisite GetPrimaryPrerequisite(const bool bAllowThis = true) const PURE_VIRTUAL(GetPrimaryPrerequisite, return FTickPrerequisite(););
	
	CONSTRAINTS_API FHandleModifiedEvent& HandleModified();
	CONSTRAINTS_API void Notify(EHandleEvent InEvent, const bool bPreTickTarget = false) const;
	mutable bool bNotifying = false;

#if WITH_EDITOR
	virtual FString GetLabel() const PURE_VIRTUAL(GetLabel, return FString(););
	virtual FString GetFullLabel() const PURE_VIRTUAL(GetFullLabel, return FString(););
#endif

	//possible bindingID
	UPROPERTY(BlueprintReadOnly, Category = "Binding")
	FMovieSceneObjectBindingID ConstraintBindingID;

private:
	FHandleModifiedEvent OnHandleModified;
};

/**
 * UTransformableComponentHandle
 */

struct FComponentEvaluationGraphBinding
{
	void OnActorMoving(AActor* InActor);
	bool bPendingFlush = false;
};

UCLASS(Blueprintable, MinimalAPI)
class UTransformableComponentHandle : public UTransformableHandle 
{
	GENERATED_BODY()
	
public:
	
	CONSTRAINTS_API virtual ~UTransformableComponentHandle();

	CONSTRAINTS_API virtual void PostLoad() override;
	
	/** Sanity check to ensure that Component. */
	CONSTRAINTS_API virtual bool IsValid(const bool bDeepCheck = true) const override;
	
	/** Sets the global transform of Component. */
	CONSTRAINTS_API virtual void SetGlobalTransform(const FTransform& InGlobal) const override;
	/** Sets the local transform of Component in it's attachment. */
	CONSTRAINTS_API virtual void SetLocalTransform(const FTransform& InLocal) const override;
	/** Gets the global transform of Component. */
	CONSTRAINTS_API virtual FTransform GetGlobalTransform() const override;
	/** Gets the local transform of Component in it's attachment. */
	CONSTRAINTS_API virtual FTransform GetLocalTransform() const override;
	/** Tick any skeletal mesh related to the component. */
	CONSTRAINTS_API virtual void TickTarget() const override;

	/**
	 * Perform any pre-evaluation of the handle to ensure that the transform data is up to date.
	 * @param bTickSkeletalMeshes to force any pre-evaluation ticking. The rig will still be pre-evaluated even
	 * if bTick is false (it just won't tick the bound skeletal meshes) Default is false.
	*/
	CONSTRAINTS_API virtual void PreEvaluate(const bool bTickSkeletalMeshes = false) const override;
	
	/** Returns the target object containing the tick function (e.i. Component). */
	CONSTRAINTS_API virtual UObject* GetPrerequisiteObject() const override;
	/** Returns Component's tick function. */
	CONSTRAINTS_API virtual FTickFunction* GetTickFunction() const override;

	/** Generates a hash value of Component. */
	CONSTRAINTS_API virtual uint32 GetHash() const override;

	/** Returns the underlying targeted object. */
	CONSTRAINTS_API virtual TWeakObjectPtr<UObject> GetTarget() const override;

	/** Check for direct dependencies (ie hierarchy) with InOther. */
	CONSTRAINTS_API virtual bool HasDirectDependencyWith(const UTransformableHandle& InOther) const override;

	/** 
	 * Look for a possible tick function that can be used as a prerequisite.
	 * If bAllowThis is false then the component's tick function is not allowed as a primary prerequisite.
	 */
	CONSTRAINTS_API virtual FTickPrerequisite GetPrimaryPrerequisite(const bool bAllowThis = true) const override;

	/** Get the array of float channels for the specified section*/
	CONSTRAINTS_API virtual TArrayView<FMovieSceneFloatChannel*>  GetFloatChannels(const UMovieSceneSection* InSection) const override;
	/** Get the array of double channels for the specified section*/
	CONSTRAINTS_API virtual TArrayView<FMovieSceneDoubleChannel*>  GetDoubleChannels(const UMovieSceneSection* InSection) const override;
	/** Add Transform Keys at the specified times*/
	CONSTRAINTS_API virtual bool AddTransformKeys(const TArray<FFrameNumber>& InFrames,
		const TArray<FTransform>& InTransforms,
		const EMovieSceneTransformChannel& InChannels,
		const FFrameRate& InTickResolution,
		UMovieSceneSection* InSection,
		const bool bLocal = true) const override;

	/** Resolve the bound objects so that any object it references are resovled and correctly set up*/
	CONSTRAINTS_API virtual void ResolveBoundObjects(FMovieSceneSequenceID LocalSequenceID, TSharedRef<UE::MovieScene::FSharedPlaybackState> SharedPlaybackState, UObject* SubObject = nullptr) override;

	/** Make a duplicate of myself with this outer*/
	CONSTRAINTS_API virtual UTransformableHandle* Duplicate(UObject* NewOuter) const override;

#if WITH_EDITOR
	/** Returns labels used for UI. */
	CONSTRAINTS_API virtual FString GetLabel() const override;
	CONSTRAINTS_API virtual FString GetFullLabel() const override;
#endif
	
	/** The Component that this handle is pointing at. */
	UPROPERTY(BlueprintReadOnly, Category = "Object")
	TWeakObjectPtr<USceneComponent> Component;

	/** Optional socket name on Component. */
	UPROPERTY(BlueprintReadOnly, Category = "Object")
	FName SocketName = NAME_None;

	/** Registers/Unregisters useful delegates to track changes in the Component's transform. */
	CONSTRAINTS_API void UnregisterDelegates() const;
	CONSTRAINTS_API void RegisterDelegates();
	
#if WITH_EDITOR
	CONSTRAINTS_API void OnActorMoving(AActor* InActor);
	CONSTRAINTS_API void OnPostPropertyChanged(UObject* InObject, FPropertyChangedEvent& InPropertyChangedEvent);
	CONSTRAINTS_API void OnObjectsReplaced(const TMap<UObject*, UObject*>& InOldToNewInstances);
#endif

protected:

	static FComponentEvaluationGraphBinding& GetEvaluationBinding();
};
