// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimationEvaluation.h"
#include "TransformableHandle.h"
#include "HAL/Platform.h"

class USceneComponent;
class USkeletalMeshComponent;
class UTransformableHandle;

namespace TransformableHandleUtils
{
	/** Returns true if the constraints' evaluation scheme should skip ticking skeletal meshes. */
	CONSTRAINTS_API bool SkipTicking();

	/** Force ticking all the skeletal meshes related to this component. */
	CONSTRAINTS_API void TickDependantComponents(USceneComponent* InComponent);
	
	/** Force ticking InSkeletalMeshComponent. */
	CONSTRAINTS_API void TickSkeletalMeshComponent(USkeletalMeshComponent* InSkeletalMeshComponent);

	/**
	 * Mark InSceneComponent for animation evaluation.
	 * If bRecursive is true, then all skeletal mesh components up the hierarchy will be marked for evaluation.
	 */
	CONSTRAINTS_API void MarkComponentForEvaluation(const USceneComponent* InSceneComponent, const bool bRecursive = false);

	UE_DEPRECATED(5.8, "Use TransformableHandleUtils::EvaluateGlobalTransform instead")
	CONSTRAINTS_API FTransform GetGlobalTransform(USceneComponent* InSceneComponent, const FName InSocketName);
	
	/**
	 * Evaluates the global transform of the scene component (composed with a socket component transform if InSocketName is not none).
	 * If any of the component's parent is a skeletal mesh and has a valid evaluator, then it will be used to compose the final transform.
	 * Note that this function will update invalid evaluators if needed.
	 */
	CONSTRAINTS_API FTransform EvaluateGlobalTransform(USceneComponent* InSceneComponent, const FName InSocketName);
	
	/**
	* Returns the global transform of the scene component (composed with a socket component transform if InSocketName is not none).
	* If the component or one of its parents is a skeletal mesh component and has an invalid evaluator, this function will return the equivalent
	* of USceneComponent::GetSocketTransform. Otherwise, the evaluators will be used to compose the final transform.
	*/
	CONSTRAINTS_API FTransform GetGlobalTransform(const USceneComponent* InSceneComponent, const FName InSocketName);
	
	/** Returns an updated version of InSceneComponent's animation evaluator. */
	CONSTRAINTS_API const UE::Anim::FAnimationEvaluator& EvaluateComponent(USceneComponent* InSceneComponent);
	
	/** Returns an updated version of InSceneComponent's animation evaluator and adds the input post-evaluation task if not already added. */
	CONSTRAINTS_API const UE::Anim::FAnimationEvaluator& EvaluateComponent(
		USceneComponent* InSceneComponent,
		const UE::Anim::FAnimationEvaluationTask& InTask);

	/**
	 * Returns a cast version of InHandle's target (if it's valid).  
	 */
	template<typename T = UObject>
	T* GetTarget(const TObjectPtr<UTransformableHandle>& InHandle)
	{
		return IsValid(InHandle) && InHandle->IsValid() ? Cast<T>(InHandle->GetTarget().Get()) : nullptr;
	}
}
