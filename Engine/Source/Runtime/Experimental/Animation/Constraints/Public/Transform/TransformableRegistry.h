// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "HAL/Platform.h"
#include "Templates/Function.h"

class UClass;
class UObject;
class UTransformableHandle;

/**
 * TransformableRegistry
 * Can be used to register customization transformable handle factory for specific objects. 
 */

class FTransformableRegistry
{
public:

	/** Defines the function that will allocate a new transformable handle to wrap the object. */
	using CreateHandleFuncT = TFunction<UTransformableHandle*(UObject*, const FName&)>;
	/** Defines the function that return a hash value from the object and the attachment name. */
	using GetHashFuncT = TFunction<uint32(const UObject*, const FName&)>;
	
	CONSTRAINTS_API ~FTransformableRegistry();

	/** Get registry singleton */
	static CONSTRAINTS_API FTransformableRegistry& Get();

	/** Registers a InClass as transformable object. */
	CONSTRAINTS_API void Register(UClass* InClass, CreateHandleFuncT InHandleFunc, GetHashFuncT InHashFunc);

	/** Get the associated hash function for that specific class. */
	CONSTRAINTS_API GetHashFuncT GetHashFunction(const UClass* InClass) const;
	/** Get the associated handle creation function for that specific class. */
	CONSTRAINTS_API CreateHandleFuncT GetCreateFunction(const UClass* InClass) const;

protected:
	friend class FConstraintsModule;

	/** Registers the basic transformable objects. (called when starting the constraints module). */
	static CONSTRAINTS_API void RegisterBaseObjects();
	/** Registers the basic transformable objects. (called when shutting down the constraints module). */
	static CONSTRAINTS_API void UnregisterAllObjects();
	
private:
	FTransformableRegistry() = default;

	/** Per class information that need to be register. */
	struct FTransformableInfo
	{
		CreateHandleFuncT CreateHandleFunc;
		GetHashFuncT GetHashFunc;
	};

	/** Find the associated info for that specific class. */
	CONSTRAINTS_API const FTransformableInfo* FindInfo(const UClass* InClass) const;

	/** List of all registered transformable objects. */
	TMap<UClass*, FTransformableInfo> Transformables;
};
