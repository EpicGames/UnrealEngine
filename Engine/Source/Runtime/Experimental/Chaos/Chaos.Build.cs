// Copyright Epic Games, Inc. All Rights Reserved.

using System.Linq;
using System.Collections.Generic;

namespace UnrealBuildTool.Rules
{
    public class Chaos : ModuleRules
    {
        public Chaos(ReadOnlyTargetRules Target) : base(Target)
        {
			NumIncludedBytesPerUnityCPPOverride = 192 * 1024; // This is half of the default unity size(NumIncludedBytesPerUnityCPP) specified in TargetRules.cs

            PublicDependencyModuleNames.AddRange(
                new string[] {
				"AutoRTFM",
                "Core",
                "CoreUObject",
				"ChaosCore",
                "IntelISPC",
				"TraceLog",
                "Voronoi",
				"GeometryCore",
				"ChaosVDRuntime",
				"TraceBasedDebuggers",
				"NNE",
				}
			);
			PrivateDependencyModuleNames.AddRange(
				new string[] {
				"Eigen",
				"MeshDescription"
				}
			);

			PublicDefinitions.Add("COMPILE_WITHOUT_UNREAL_SUPPORT=0");
			PrivateDefinitions.Add("CHAOS_INCLUDE_LEVEL_1=1");

			PrivateIncludePathModuleNames.Add("PhysicsCore"); // Needed because we include "ChaosEngineInterface.h"

			if (Target.bUseChaosMemoryTracking == true)
			{
				PublicDefinitions.Add("CHAOS_MEMORY_TRACKING=1");
			}
			else
			{
				PublicDefinitions.Add("CHAOS_MEMORY_TRACKING=0");
			}

			SetupModuleChaosVisualDebuggerSupport(Target);
			
			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

			StaticAnalyzerDisabledCheckers.Add("cplusplus.NewDeleteLeaks"); // To be reevalulated, believed to be invalid warnings.
			StaticAnalyzerDisabledCheckers.Add("core.UndefinedBinaryOperatorResult"); // Invalid warning in mass property calculation.
		}
    }
}
