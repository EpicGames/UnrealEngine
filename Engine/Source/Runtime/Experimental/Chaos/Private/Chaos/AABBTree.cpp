// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/AABBTree.h"

int32 FAABBTreeCVars::UpdateDirtyElementPayloadData = 1;
FAutoConsoleVariableRef FAABBTreeCVars::CVarUpdateDirtyElementPayloadData(TEXT("p.aabbtree.updatedirtyelementpayloads"), FAABBTreeCVars::UpdateDirtyElementPayloadData, TEXT("Allow AABB tree elements to update internal payload data when they receive a payload update"));

int32 FAABBTreeDirtyGridCVars::DirtyElementGridCellSize = 1000; // 0 means disabled
FAutoConsoleVariableRef FAABBTreeDirtyGridCVars::CVarDirtyElementGridCellSize(TEXT("p.aabbtree.DirtyElementGridCellSize"), FAABBTreeDirtyGridCVars::DirtyElementGridCellSize, TEXT("DirtyElement Grid acceleration structure cell size in cm. 0 or less will disable the feature"));

int32 FAABBTreeDirtyGridCVars::DirtyElementMaxGridCellQueryCount = 340;
FAutoConsoleVariableRef FAABBTreeDirtyGridCVars::CVarDirtyElementMaxGridCellQueryCount(TEXT("p.aabbtree.DirtyElementMaxGridCellQueryCount"), FAABBTreeDirtyGridCVars::DirtyElementMaxGridCellQueryCount, TEXT("Maximum grid cells to query (per raycast for example) in DirtyElement grid acceleration structure before falling back to brute force"));

int32 FAABBTreeDirtyGridCVars::DirtyElementMaxPhysicalSizeInCells = 16;
FAutoConsoleVariableRef FAABBTreeDirtyGridCVars::CVarDirtyElementMaxPhysicalSizeInCells(TEXT("p.aabbtree.DirtyElementMaxPhysicalSizeInCells"), FAABBTreeDirtyGridCVars::DirtyElementMaxPhysicalSizeInCells, TEXT("If a dirty element stradles more than this number of cells, it will no be added to the grid acceleration structure"));

int32 FAABBTreeDirtyGridCVars::DirtyElementMaxCellCapacity = 32;
FAutoConsoleVariableRef FAABBTreeDirtyGridCVars::CVarDirtyElementMaxCellCapacity(TEXT("p.aabbtree.DirtyElementMaxCellCapacity"), FAABBTreeDirtyGridCVars::DirtyElementMaxCellCapacity, TEXT("The maximum number of dirty elements that can be added to a single grid cell before spilling to slower flat list"));

CSV_DEFINE_CATEGORY(ChaosPhysicsTimers, true);

int32 FAABBTreeCVars::SplitAtAverageCenter = 1;
FAutoConsoleVariableRef FAABBTreeCVars::CVarSplitAtAverageCenter(TEXT("p.aabbtree.splitataveragecenter"), FAABBTreeCVars::SplitAtAverageCenter, TEXT("Split AABB tree nodes at the average of the element centers"));

int32 FAABBTreeCVars::SplitOnVarianceAxis = 1;
FAutoConsoleVariableRef FAABBTreeCVars::CVarSplitOnVarianceAxis(TEXT("p.aabbtree.splitonvarianceaxis"), FAABBTreeCVars::SplitOnVarianceAxis, TEXT("Split AABB tree nodes along the axis with the largest element center variance"));

float FAABBTreeCVars::DynamicTreeBoundingBoxPadding = 5.0f;
FAutoConsoleVariableRef FAABBTreeCVars::CVarDynamicTreeBoundingBoxPadding(TEXT("p.aabbtree.DynamicTreeBoundingBoxPadding"), FAABBTreeCVars::DynamicTreeBoundingBoxPadding, TEXT("Additional padding added to bounding boxes for dynamic AABB trees to amortize update cost"));

int32 FAABBTreeCVars::DynamicTreeLeafCapacity = 8;
FAutoConsoleVariableRef FAABBTreeCVars::CVarDynamicTreeLeafCapacity(TEXT("p.aabbtree.DynamicTreeLeafCapacity"), FAABBTreeCVars::DynamicTreeLeafCapacity, TEXT("Dynamic Tree Leaf Capacity"));

float FAABBTreeCVars::DynamicTreeLeafEnlargePercent = 0.1f;
FAutoConsoleVariableRef FAABBTreeCVars::CVarDynamicTreeLeafEnlargePercent(TEXT("p.aabbtree.DynamicTreeLeafEnlargePercent"), FAABBTreeCVars::DynamicTreeLeafEnlargePercent, TEXT("Additional percentage added to bounding box of the leaf to amortize update cost. Ex: set 0.05 for 5%"));

bool FAABBTimeSliceCVars::bUseTimeSliceMillisecondBudget = true;
FAutoConsoleVariableRef FAABBTimeSliceCVars::CVarUseTimeSliceByMillisecondBudget(TEXT("p.aabbtree.UseTimeSliceMillisecondBudget"), FAABBTimeSliceCVars::bUseTimeSliceMillisecondBudget, TEXT("Set to True if we want to timeslice tree generation by a milisecond budget instead of per nodes processed"));

float FAABBTimeSliceCVars::MaxProcessingTimePerSliceSeconds = 0.001f;
FAutoConsoleVariableRef FAABBTimeSliceCVars::CVarMaxProcessingTimePerSlice(TEXT("p.aabbtree.MaxProcessingTimePerSliceSeconds"), FAABBTimeSliceCVars::MaxProcessingTimePerSliceSeconds, TEXT("Set to True if we want to timeslice tree generation by a milisecond budget instead of per nodes processed or data size copied"));

int32 FAABBTimeSliceCVars::MinNodesChunkToProcessBetweenTimeChecks = 250;
FAutoConsoleVariableRef FAABBTimeSliceCVars::CVarMinNodesChunkToProcessBetweenTimeChecks(TEXT("p.aabbtree.MinNodesChunkToProcessBetweenTimeChecks"), FAABBTimeSliceCVars::MinNodesChunkToProcessBetweenTimeChecks, TEXT("Minimum amount of nodes we want to process before checking if we are withing budget"));

int32 FAABBTimeSliceCVars::MinDataChunkToProcessBetweenTimeChecks = 500;
FAutoConsoleVariableRef FAABBTimeSliceCVars::CVarMinDataChunkToProcessBetweenTimeChecks(TEXT("p.aabbtree.MinDataChunkToProcessBetweenTimeChecks"), FAABBTimeSliceCVars::MinDataChunkToProcessBetweenTimeChecks, TEXT("Minimum amount of data elements to process before checking if we are withing budget"));
