// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/AsyncInitBodyHelper.h"
#include "HAL/IConsoleManager.h"

namespace Chaos
{
	namespace CVars
	{
		CHAOS_API bool bEnableAsyncInitBody = false;

		FAutoConsoleVariableRef CVarEnableAsyncInitBody(
		TEXT("p.Chaos.EnableAsyncInitBody"),
		bEnableAsyncInitBody,
		TEXT("[Experimental] Master switch for async body instance creation AND destruction via FPhysScene_AsyncPhysicsStateJobQueue. When true, body init/term work runs on worker threads against pre-cached inputs (cache populated on the game thread). Read-only - must be set in DefaultEngine.ini before world init. Default is false."),
		ECVF_ReadOnly);
	}
}