// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/BVHParticles.h"
#include "Chaos/BoundingVolumeHierarchy.h"

namespace Chaos
{
	FBVHParticles::FBVHParticles()
		: FParticles()
		, MBVH(new TBoundingVolumeHierarchy<FParticles, TArray<int32>>(*this, CollisionParticlesBVHDepth))
	{}

	FBVHParticles::FBVHParticles(FBVHParticles&& Other)
		: FParticles(MoveTemp(Other))
		, MBVH(new TBoundingVolumeHierarchy<FParticles, TArray<int32>>(MoveTemp(*Other.MBVH)))
	{}

	FBVHParticles::FBVHParticles(FParticles&& Other)
		: FParticles(MoveTemp(Other))
		, MBVH(new TBoundingVolumeHierarchy<FParticles, TArray<int32>>(*this, CollisionParticlesBVHDepth))
	{}

	FBVHParticles::~FBVHParticles()
	{
		delete MBVH;
	}

	FBVHParticles& FBVHParticles::operator=(const FBVHParticles& Other)
	{
		*this = FBVHParticles(Other);
		return *this;
	}

	FBVHParticles& FBVHParticles::operator=(FBVHParticles&& Other)
	{
		*MBVH = MoveTemp(*Other.MBVH);
		FParticles::operator=(static_cast<FParticles&&>(Other));
		return *this;
	}

	FBVHParticles::FBVHParticles(const FBVHParticles& Other)
		: FParticles()
	{
		AddParticles(Other.Size());
		for(int32 i = Other.Size() - 1; 0 <= i; i--)
		{
			SetX(i, Other.GetX(i));
		}
		MBVH = new TBoundingVolumeHierarchy<FParticles, TArray<int32>>(*this, CollisionParticlesBVHDepth);
	}

	void FBVHParticles::UpdateAccelerationStructures()
	{
		MBVH->UpdateHierarchy();
	}

	const TArray<int32> FBVHParticles::FindAllIntersections(const FAABB3& Object) const
	{
		return MBVH->FindAllIntersections(Object);
	}

	void FBVHParticles::Serialize(FChaosArchive& Ar)
	{
		FParticles::Serialize(Ar);
		Ar << *MBVH;
	}
}
