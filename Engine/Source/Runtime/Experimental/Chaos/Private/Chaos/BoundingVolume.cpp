// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/BoundingVolume.h"
#include "Chaos/AABBTree.h"
#include "UObject/ExternalPhysicsCustomObjectVersion.h"

int FBoundingVolumeCVars::FilterFarBodies = 0;

FAutoConsoleVariableRef FBoundingVolumeCVars::CVarFilterFarBodies(
    TEXT("p.RemoveFarBodiesFromBVH"),
    FBoundingVolumeCVars::FilterFarBodies,
    TEXT("Removes bodies far from the scene from the bvh\n")
        TEXT("0: Kept, 1: Removed"),
    ECVF_Default);

namespace Chaos
{
#if WITH_EDITOR
	CHAOS_API int32 MaxDirtyElements = MAX_int32;
#else
	CHAOS_API int32 MaxDirtyElements = 10000;
#endif

	FAutoConsoleVariableRef CVarMaxDirtyElements(
	    TEXT("p.MaxDirtyElements"),
	    MaxDirtyElements,
	    TEXT("The max number of dirty elements. This forces a flush which is very expensive"));

#if !UE_MERGED_MODULES
	template class CHAOS_API Chaos::TBoundingVolume<int32, FReal, 3>;
	template class CHAOS_API Chaos::TBoundingVolume<FAccelerationStructureHandle, FReal, 3>;
#endif
}
