// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/BoundingVolumeHierarchy.h"
#include "Chaos/BoundingVolume.h"
#include "Chaos/BoundingVolumeUtilities.h"
#include "Chaos/ImplicitObjectBVH.h"
#include "Chaos/Tetrahedron.h"
#include "ChaosStats.h"

namespace Chaos
{
	// Bounds expansion (for speculative contact creation) is limited by this factor multiplied by the objects size.
	// This prevents objects that are moving large distances in one frame from having bounds which overlap too many objects. 
	// We only detect collisions at the final position anyway (when CCD is disabled) so having a bounds which is much larger 
	// than the object doesn't help much. E.g., if an object moves more than its size per frame it will encounter tunneling 
	// problems when colliding against zero-thickness surfaces such as tri meshes, regardless of the bounds size.
	// NOTE: This is only used by objects that have CCD disabled.
	FRealSingle Chaos_Bounds_MaxInflationScale = 2.f;
	FAutoConsoleVariableRef CVarBoundsMaxInflatiuonScale(TEXT("p.Chaos.MaxInflationScale"), Chaos_Bounds_MaxInflationScale, TEXT("A limit on the bounds used to detect collisions when CCD is disabled. The bounds limit is this scale multiplied by the object's max dimension"));


	template<class OBJECT_ARRAY, class LEAF_TYPE, class T, int d>
	TBoundingVolumeHierarchy<OBJECT_ARRAY, LEAF_TYPE, T, d>::TBoundingVolumeHierarchy(const OBJECT_ARRAY& Objects, const int32 MaxLevels, const bool bUseVelocity, const T Dt)
		: MObjects(&Objects), MMaxLevels(MaxLevels)
	{
		if(GetObjectCount(Objects) > 0)
		{
			UpdateHierarchy(DefaultAllowMultipleSplitting, bUseVelocity, DefaultDt);
		}
	}

	template<class OBJECT_ARRAY, class LEAF_TYPE, class T, int d>
	TBoundingVolumeHierarchy<OBJECT_ARRAY, LEAF_TYPE, T, d>::TBoundingVolumeHierarchy(const OBJECT_ARRAY& Objects, const TArray<uint32>& ActiveIndices, const int32 MaxLevels, const bool bUseVelocity, const T Dt)
		: MObjects(&Objects), MMaxLevels(MaxLevels)
	{
		if(GetObjectCount(Objects) > 0)
		{
			UpdateHierarchy(ActiveIndices, DefaultAllowMultipleSplitting, bUseVelocity, DefaultDt);
		}
	}

	template<class OBJECT_ARRAY, class LEAF_TYPE, class T, int d>
	const TAABB<T, d>& TBoundingVolumeHierarchy<OBJECT_ARRAY, LEAF_TYPE, T, d>::GetWorldSpaceBoundingBox(const TGeometryParticles<T, d>& InParticles, const int32 Index)
	{
		return Chaos::GetWorldSpaceBoundingBox(InParticles, Index, MWorldSpaceBoxes);
	}

#if !UE_BUILD_SHIPPING
	template<typename T, int d>
	void DrawNodeRecursive(ISpacialDebugDrawInterface<T>* InInterface, const TBVHNode<T, d>& InNode, const TArray<TBVHNode<T, d>>& InAllNodes)
	{
		TAABB<T, d> Box(InNode.MMin, InNode.MMax);

		InInterface->Box(Box, { 1.0f, 1.0f, 1.0f }, 1.0f);

		for(const int32 Child : InNode.MChildren)
		{
			DrawNodeRecursive(InInterface, InAllNodes[Child], InAllNodes);
		}
	}

	template<class OBJECT_ARRAY, class LEAF_TYPE, class T, int d>
	void Chaos::TBoundingVolumeHierarchy<OBJECT_ARRAY, LEAF_TYPE, T, d>::DebugDraw(ISpacialDebugDrawInterface<T>* InInterface) const
	{
		if(Elements.Num() > 0)
		{
			DrawNodeRecursive(InInterface, Elements[0], Elements);
		}
	}
#endif

	template<class OBJECT_ARRAY, class T, int d, typename TPayloadType>
	TArray<int32> MakeNewLeaf(const OBJECT_ARRAY& Objects, const TArray<int32>& AllObjects, const TArray<int32>* /*Type*/)
	{
		return TArray<int32>(AllObjects);
	}

	template<class OBJECT_ARRAY, class T, int d, typename TPayloadType>
	TBoundingVolume<TPayloadType, T, d> MakeNewLeaf(const OBJECT_ARRAY& Objects, const TArray<int32>& AllObjects, const TBoundingVolume<TPayloadType, T, d>* /*Type*/)
	{
#if CHAOS_PARTICLEHANDLE_TODO
		// @todo(mlentine): This is pretty dirty but should work for now.
		return TBoundingVolume<OBJECT_ARRAY, TPayloadType, T, d>(Objects, reinterpret_cast<const TArray<uint32>&>(AllObjects));
#else
		TArray<TSOAView<OBJECT_ARRAY>> SOAs;
		SOAs.Add(TSOAView<OBJECT_ARRAY>(const_cast<OBJECT_ARRAY*>(&Objects)));	//todo: this const_cast is here because bvh is holding on to things as const but giving it out as non const. Not sure if we should change API or not
		//return TBoundingVolume<OBJECT_ARRAY, T, d>(MakeParticleView<OBJECT_ARRAY>({ {&Objects} }));
		return TBoundingVolume<TPayloadType, T, d>(MakeParticleView<OBJECT_ARRAY>(MoveTemp(SOAs)));
#endif
	}

	template<class OBJECT_ARRAY, class LEAF_TYPE, class T, int d>
	void TBoundingVolumeHierarchy<OBJECT_ARRAY, LEAF_TYPE, T, d>::UpdateHierarchy(const bool bAllowMultipleSplitting, const bool bUseVelocity, const T Dt)
	{
		const int32 NumObjects = GetObjectCount(*MObjects);

		MScratchAllObjects.Reset();
		Leafs.Reset();

		MGlobalObjects.Reset();
		for(int32 Idx = 0; Idx < NumObjects; ++Idx)	//todo: check for disabled objects if possible? (this is what BV does)
		{
			if(IsDisabled(*MObjects, Idx))
			{
				continue;
			}

			if(HasBoundingBox(*MObjects, Idx))
			{
				MScratchAllObjects.Add(Idx);
			}
			else
			{
				MGlobalObjects.Add(Idx);
			}
		}

		UpdateHierarchyImp(MScratchAllObjects, bAllowMultipleSplitting, bUseVelocity, Dt);
	}

	template<class OBJECT_ARRAY, class LEAF_TYPE, class T, int d>
	void TBoundingVolumeHierarchy<OBJECT_ARRAY, LEAF_TYPE, T, d>::UpdateHierarchy(const TArray<uint32>& ActiveIndices, const bool bAllowMultipleSplitting, const bool bUseVelocity, const T Dt)
	{
		const int32 NumObjects = ActiveIndices.Num();

		MScratchAllObjects.Reset();
		MScratchAllObjects.Reserve(NumObjects);

		MGlobalObjects.Reset();
		for(uint32 Idx : ActiveIndices)
		{
			check(!IsDisabled(*MObjects, Idx));
			if(HasBoundingBox(*MObjects, Idx))
			{
				MScratchAllObjects.Add(Idx);
			}
			else
			{
				MGlobalObjects.Add(Idx);
			}
		}

		UpdateHierarchyImp(MScratchAllObjects, bAllowMultipleSplitting, bUseVelocity, Dt);
	}

	template<class OBJECT_ARRAY, class LEAF_TYPE, class T, int d>
	void TBoundingVolumeHierarchy<OBJECT_ARRAY, LEAF_TYPE, T, d>::UpdateHierarchyImp(const TArray<int32>& AllObjects, const bool bAllowMultipleSplitting, const bool bUseVelocity, const T Dt)
	{
		Elements.Reset();
		MWorldSpaceBoxes.Reset();

		if(!AllObjects.Num())
		{
			return;
		}
		ComputeAllWorldSpaceBoundingBoxes(*MObjects, AllObjects, bUseVelocity, Dt, MWorldSpaceBoxes);

		int32 Axis;
		const TAABB<T, d> GlobalBox = ComputeGlobalBoxAndSplitAxis(*MObjects, AllObjects, MWorldSpaceBoxes, bAllowMultipleSplitting, Axis);

		TBVHNode<T, d> RootNode;
		RootNode.MMin = GlobalBox.Min();
		RootNode.MMax = GlobalBox.Max();
		RootNode.MAxis = Axis;
		RootNode.LeafIndex = INDEX_NONE;
		Elements.Add(RootNode);
		if(AllObjects.Num() > MIN_NUM_OBJECTS) // TODO(mlentine): What is a good number to stop at?
		{
			int32 StartIndex = GenerateNextLevel(GlobalBox.Min(), GlobalBox.Max(), AllObjects, Axis, 1, bAllowMultipleSplitting, Elements, Leafs);
			for(int32 i = 0; i < (Axis == -1 ? 8 : 2); i++)
			{
				Elements[0].MChildren.Add(StartIndex + i);
			}
		}
		else
		{
			Elements[0].LeafIndex = Leafs.Num();
			Leafs.Add(MakeNewLeaf<OBJECT_ARRAY, T, d, TPayloadType>(*MObjects, AllObjects, static_cast<LEAF_TYPE*>(nullptr)));
		}
		check(Elements[0].LeafIndex < Leafs.Num() || Elements[0].MChildren.Num() > 0);
		UE_LOGF(LogChaos, Verbose, "Generated Tree with %d Nodes", Elements.Num());
		//PrintTree("", &Elements[0]);
	}

	template<class OBJECT_ARRAY, typename TPayloadType, class T, int d>
	TArray<int32> FindAllIntersectionsLeafHelper(const TArray<int32>& Leaf, const TVector<T, d>& Point)
	{
		return Leaf;
	}

	template<class OBJECT_ARRAY, typename TPayloadType, class T, int d>
	TArray<int32> FindAllIntersectionsLeafHelper(const TBoundingVolume<TPayloadType, T, d>& Leaf, const TVector<T, d>& Point)
	{
#if CHAOS_PARTICLEHANDLE_TODO
		return Leaf.FindAllIntersections(Point);
#else
		check(false);
		return TArray<int32>();
#endif
	}

	template<class OBJECT_ARRAY, typename TPayloadType, class T, int d>
	TArray<int32> TBoundingVolumeHierarchy<OBJECT_ARRAY, TPayloadType, T, d>::FindAllIntersectionsHelper(const TBVHNode<T, d>& MyNode, const TVector<T, d>& Point) const
	{
		TAABB<T, d> MBox(MyNode.MMin, MyNode.MMax);
		if(MBox.SignedDistance(Point) > 0)
		{
			return TArray<int32>();
		}
		if(MyNode.MChildren.Num() == 0)
		{
			return FindAllIntersectionsLeafHelper<OBJECT_ARRAY, TPayloadType, T, d>(Leafs[MyNode.LeafIndex], Point);
		}
		const TVector<T, d> ObjectCenter = MBox.Center();
		int32 Child = 0;
		if(MyNode.MAxis >= 0)
		{
			if(Point[MyNode.MAxis] > ObjectCenter[MyNode.MAxis])
				Child += 1;
		}
		else
		{
			if(Point[0] > ObjectCenter[0])
				Child += 1;
			if(Point[1] > ObjectCenter[1])
				Child += 2;
			if(Point[2] > ObjectCenter[2])
				Child += 4;
		}
		return FindAllIntersectionsHelper(Elements[MyNode.MChildren[Child]], Point);
	}

	template <typename T, int d>
	bool IntersectsHelper(const TAABB<T, d>& WorldSpaceBox, const TAABB<T, d>& LocalBox)
	{
		return WorldSpaceBox.Intersects(LocalBox);
	}

	template <typename T, int d>
	bool IntersectsHelper(const TAABB<T, d>& WorldSpaceBox, const TSpatialRay<T, d>& Ray)
	{
		//Very slow, should look at using slab algorithm
		const FBox VolumeBox(WorldSpaceBox.Min(), WorldSpaceBox.Max());
		FVector Hit;
		FVector Normal;
		FRealSingle Time;
		return FMath::LineExtentBoxIntersection(VolumeBox, Ray.Start, Ray.End, FVector::ZeroVector, Hit, Normal, Time);

		//return WorldSpaceBox.FindClosestIntersection(Ray.Start, Ray.End, 0).Second;
	}

	template <typename OBJECT_ARRAY>
	struct TSpecializeParticlesHelper
	{
		template <typename T, int d, typename QUERY_OBJECT>
		static void AccumulateChildrenResults(TArray<int32>& AccumIntersectionList, const TArray<int32>& PotentialChildren, const QUERY_OBJECT& QueryObject, const TMap<int32, TAABB<T, d>>& WorldSpaceBoxes)
		{
			for(int32 ChildIndex : PotentialChildren)
			{
				if(IntersectsHelper(WorldSpaceBoxes[ChildIndex], QueryObject)) //todo(ocohen): actually just a single point so should call Contains
				{
					AccumIntersectionList.Add(ChildIndex);
				}
			}
		}
	};

	int32 CheckBox = 1;
	FAutoConsoleVariableRef CVarCheckBox(TEXT("p.checkbox"), CheckBox, TEXT(""));

	template <typename T, int d>
	struct TSpecializeParticlesHelper<TParticles<T, d>>
	{
		template <typename QUERY_OBJECT>
		static void AccumulateChildrenResults(TArray<int32>& AccumIntersectionList, const TArray<int32>& PotentialChildren, const QUERY_OBJECT& QueryObject, const TMap<int32, TAABB<T, d>>& WorldSpaceBoxes)
		{
			if(CheckBox)
			{
				for(int32 ChildIndex : PotentialChildren)
				{
					if(IntersectsHelper(WorldSpaceBoxes[ChildIndex], QueryObject))	//todo(ocohen): actually just a single point so should call Contains
					{
						AccumIntersectionList.Add(ChildIndex);
					}
				}
			}
			else
			{
				AccumIntersectionList.Append(PotentialChildren);
			}
		}
	};

	template<class OBJECT_ARRAY, typename TPayloadType, typename T, int d, typename QUERY_OBJECT>
	TArray<int32> FindAllIntersectionsLeafHelper(const TArray<int32>& Leaf, const QUERY_OBJECT& QueryObject)
	{
		return Leaf;
	}

	template<class OBJECT_ARRAY, typename TPayloadType, class T, int d, typename QUERY_OBJECT>
	TArray<int32> FindAllIntersectionsLeafHelper(const TBoundingVolume<TPayloadType, T, d>& Leaf, const QUERY_OBJECT& QueryObject)
	{
#if CHAOS_PARTICLEHANDLE_TODO
		return Leaf.FindAllIntersectionsImp(QueryObject);
#else
		return TArray<int32>();
#endif
	}

	int32 FindAllIntersectionsSingleThreaded = 1;
	FAutoConsoleVariableRef CVarFindAllIntersectionsSingleThreaded(TEXT("p.FindAllIntersectionsSingleThreaded"), FindAllIntersectionsSingleThreaded, TEXT(""));

	template<class OBJECT_ARRAY, class LEAF_TYPE, class T, int d>
	template <typename QUERY_OBJECT>
	void TBoundingVolumeHierarchy<OBJECT_ARRAY, LEAF_TYPE, T, d>::FindAllIntersectionsHelperRecursive(const TBVHNode<T, d>& MyNode, const QUERY_OBJECT& QueryObject, TArray<int32>& AccumulateElements) const
	{
		TAABB<T, d> MBox(MyNode.MMin, MyNode.MMax);
		if(!IntersectsHelper(MBox, QueryObject))
		{
			return;
		}
		if(MyNode.MChildren.Num() == 0)
		{
			TSpecializeParticlesHelper<OBJECT_ARRAY>::AccumulateChildrenResults(AccumulateElements, FindAllIntersectionsLeafHelper<OBJECT_ARRAY, TPayloadType, T, d>(Leafs[MyNode.LeafIndex], QueryObject), QueryObject, MWorldSpaceBoxes);
			return;
		}

		int32 NumChildren = MyNode.MChildren.Num();
		for(int32 Child = 0; Child < NumChildren; ++Child)
		{
			FindAllIntersectionsHelperRecursive(Elements[MyNode.MChildren[Child]], QueryObject, AccumulateElements);
		}
	}

	int32 UseAccumulationArray = 1;
	FAutoConsoleVariableRef CVarUseAccumulationArray(TEXT("p.UseAccumulationArray"), UseAccumulationArray, TEXT(""));

	template<class OBJECT_ARRAY, class LEAF_TYPE, class T, int d>
	TArray<int32> TBoundingVolumeHierarchy<OBJECT_ARRAY, LEAF_TYPE, T, d>::FindAllIntersectionsHelper(const TBVHNode<T, d>& MyNode, const TAABB<T, d>& ObjectBox) const
	{
		TArray<int32> IntersectionList;
		FindAllIntersectionsHelperRecursive(MyNode, ObjectBox, IntersectionList);

		IntersectionList.Sort();

		for(int32 i = IntersectionList.Num() - 1; i > 0; i--)
		{
			if(IntersectionList[i] == IntersectionList[i - 1])
			{
				IntersectionList.RemoveAtSwap(i, EAllowShrinking::No);
			}
		}

		return IntersectionList;
	}

	template<class OBJECT_ARRAY, class LEAF_TYPE, class T, int d>
	TArray<int32> TBoundingVolumeHierarchy<OBJECT_ARRAY, LEAF_TYPE, T, d>::FindAllIntersectionsHelper(const TBVHNode<T, d>& MyNode, const TSpatialRay<T, d>& Ray) const
	{
		TArray<int32> IntersectionList;
		FindAllIntersectionsHelperRecursive(MyNode, Ray, IntersectionList);

		IntersectionList.Sort();

		for(int32 i = IntersectionList.Num() - 1; i > 0; i--)
		{
			if(IntersectionList[i] == IntersectionList[i - 1])
			{
				IntersectionList.RemoveAtSwap(i, EAllowShrinking::No);
			}
		}

		return IntersectionList;
	}

	template<class OBJECT_ARRAY, class LEAF_TYPE, class T, int d>
	TArray<int32> TBoundingVolumeHierarchy<OBJECT_ARRAY, LEAF_TYPE, T, d>::FindAllIntersections(const TGeometryParticles<T, d>& InParticles, const int32 i) const
	{
		return FindAllIntersections(Chaos::GetWorldSpaceBoundingBox(InParticles, i, MWorldSpaceBoxes));
	}

	template <int d>
	struct FSplitCount
	{
		FSplitCount()
		{
			for(int i = 0; i < d; ++i)
			{
				Neg[i] = 0;
				Pos[i] = 0;
			}
		}

		int32 Neg[d];
		int32 Pos[d];
	};

	template <typename T, int d>
	void AccumulateNextLevelCount(const TAABB<T, d>& Box, const TVector<T, d>& MidPoint, FSplitCount<d>& Counts)
	{
		//todo(ocohen): particles min = max so avoid extra work
		for(int32 i = 0; i < d; ++i)
		{
			Counts.Neg[i] += (Box.Max()[i] < MidPoint[i] || Box.Min()[i] < MidPoint[i]) ? 1 : 0;
			Counts.Pos[i] += (Box.Min()[i] > MidPoint[i] || Box.Max()[i] > MidPoint[i]) ? 1 : 0;
		}
	}

	template<class OBJECT_ARRAY, class LEAF_TYPE, class T, int d>
	int32 TBoundingVolumeHierarchy<OBJECT_ARRAY, LEAF_TYPE, T, d>::GenerateNextLevel(const TVector<T, d>& GlobalMin, const TVector<T, d>& GlobalMax, const TArray<int32>& Objects, const int32 Axis, const int32 Level, const bool AllowMultipleSplitting, TArray<TBVHNode<T, d>>& OutElements, TArray<LEAF_TYPE>& OutLeafs)
	{
		if(Axis == -1)
		{
			return GenerateNextLevel(GlobalMin, GlobalMax, Objects, Level, OutElements, OutLeafs);
		}

		TBVHNode<T, d> LocalElements[2];
		TArray<int32> LocalObjects[2];
		LEAF_TYPE LocalLeafs[2];

		TAABB<T, d> GlobalBox(GlobalMin, GlobalMax);
		const TVector<T, d> WorldCenter = GlobalBox.Center();

		// We are splitting the node at WorldCenter along axis Axis.
		// The two child nodes will split at NextLevelCenter[0] and NextLevelCenter[1] along an axis we have yet to determine.
		TAABB<T, d> NextLevelBounds[2];
		TVector<T, d> NextLevelCenters[2];
		FSplitCount<d> NextLevelCounts[2];
		for (int32 i = 0; i < 2; ++i)
		{
			TVector<T, d> Min = GlobalBox.Min();
			TVector<T, d> Max = GlobalBox.Max();
			if (i == 0)
			{
				Max[Axis] = WorldCenter[Axis];
			}
			else
			{
				Min[Axis] = WorldCenter[Axis];
			}

			NextLevelBounds[i] = TAABB<T, d>(Min, Max);
			NextLevelCenters[i] = T(0.5) * (Min + Max);
		}

		// Determine which side of the divide each body falls. While we are here, we accumulate the number of objects
		// on each side of the child nodes would be for each of the 3 axis choices.
		for(int32 i = 0; i < Objects.Num(); ++i)
		{
			check(Objects[i] >= 0 && Objects[i] < GetObjectCount(*MObjects));
			const TAABB<T, d>& ObjectBox = Chaos::GetWorldSpaceBoundingBox(*MObjects, Objects[i], MWorldSpaceBoxes);

			// This side of the divide do we fall on? It could be both sides if we span the split point
			bool MinA = false;
			bool MaxA = false;
			if(ObjectBox.Min()[Axis] < WorldCenter[Axis])
			{
				MinA = true;
			}
			if(ObjectBox.Max()[Axis] >= WorldCenter[Axis])
			{
				MaxA = true;
			}

			// If this is not true we have a nan and the object gets ignored
			ensure(MinA || MaxA);

			// Add the object to the selected child node object list and determine where it 
			// would fall in the next level's split, for all possible split axes
			if(MinA)
			{
				LocalObjects[0].Add(Objects[i]);
				AccumulateNextLevelCount(ObjectBox, NextLevelCenters[0], NextLevelCounts[0]);
			}
			if(MaxA)
			{
				LocalObjects[1].Add(Objects[i]);
				AccumulateNextLevelCount(ObjectBox, NextLevelCenters[1], NextLevelCounts[1]);
			}
		}
		// Build each child subtree into its own local arrays in parallel, then merge them into
		// OutElements/OutLeafs in a fixed child order. This produces
		// the same ordering a single-threaded build would, while keeping the parallelism.
		TArray<TBVHNode<T, d>> ChildElements[2];
		TArray<LEAF_TYPE> ChildLeafs[2];
		int32 ChildStartIndex[2] = { INDEX_NONE, INDEX_NONE };
		PhysicsParallelFor(2,
		[this, &LocalElements, &LocalObjects, &LocalLeafs, &ChildStartIndex, &ChildElements, &ChildLeafs, &Objects, Level, &NextLevelBounds, &NextLevelCounts, &AllowMultipleSplitting]
		(int32 GroupIndex)
		{
			LocalElements[GroupIndex].MMin = NextLevelBounds[GroupIndex].Min();
			LocalElements[GroupIndex].MMax = NextLevelBounds[GroupIndex].Max();
			LocalElements[GroupIndex].MAxis = -1;
			LocalElements[GroupIndex].LeafIndex = INDEX_NONE;
			if(LocalObjects[GroupIndex].Num() > MIN_NUM_OBJECTS && Level < MMaxLevels && LocalObjects[GroupIndex].Num() < Objects.Num())
			{
				//we pick the axis that gives us the most culled even in the case when it goes in the wrong direction (GroupIndex.e the biggest min)
				int32 BestAxis = 0;
				int32 MaxCulled = 0;
				for(int32 LocalAxis = 0; LocalAxis < d; ++LocalAxis)
				{
					int32 CulledWorstCase = FMath::Min(NextLevelCounts[GroupIndex].Neg[LocalAxis], NextLevelCounts[GroupIndex].Pos[LocalAxis]);
					if(CulledWorstCase > MaxCulled)
					{
						MaxCulled = CulledWorstCase;
						BestAxis = LocalAxis;
					}
				}

				//todo(ocohen): use multi split when counts are very close
				int32 NextAxis = BestAxis;
				LocalElements[GroupIndex].MAxis = NextAxis;
				ChildStartIndex[GroupIndex] = GenerateNextLevel(
					LocalElements[GroupIndex].MMin,
					LocalElements[GroupIndex].MMax,
					LocalObjects[GroupIndex], NextAxis,
					Level + 1,
					AllowMultipleSplitting,
					ChildElements[GroupIndex],
					ChildLeafs[GroupIndex]);
			}
			else
			{
				LocalLeafs[GroupIndex] = MakeNewLeaf<OBJECT_ARRAY, T, d, TPayloadType>(
					*MObjects,
					LocalObjects[GroupIndex],
					static_cast<LEAF_TYPE*>(nullptr));
			}
		});

		// Deterministic merge of the child subtrees (child 0 then child 1), remapping their local node
		// and leaf indices, then append this level's two nodes.
		for(int32 GroupIndex = 0; GroupIndex < 2; ++GroupIndex)
		{
			if(ChildStartIndex[GroupIndex] != INDEX_NONE)
			{
				const int32 ElemOffset = OutElements.Num();
				const int32 LeafOffset = OutLeafs.Num();
				for(TBVHNode<T, d>& ChildNode : ChildElements[GroupIndex])
				{
					if(ChildNode.MChildren.Num() > 0)
					{
						for(int32& GrandChildIndex : ChildNode.MChildren)
						{
							GrandChildIndex += ElemOffset;
						}
					}
					else
					{
						ChildNode.LeafIndex += LeafOffset;
					}
				}
				OutElements.Append(MoveTemp(ChildElements[GroupIndex]));
				OutLeafs.Append(MoveTemp(ChildLeafs[GroupIndex]));

				const int32 ChildRoot = ChildStartIndex[GroupIndex] + ElemOffset;
				for(int32 j = 0; j < (LocalElements[GroupIndex].MAxis == -1 ? 8 : 2); j++)
				{
					LocalElements[GroupIndex].MChildren.Add(ChildRoot + j);
				}
			}
			else
			{
				LocalElements[GroupIndex].LeafIndex = OutLeafs.Num();
				OutLeafs.Add(MoveTemp(LocalLeafs[GroupIndex]));
			}
			check(LocalElements[GroupIndex].LeafIndex < OutLeafs.Num() || LocalElements[GroupIndex].MChildren.Num() > 0);
		}
		const int32 MinElem = OutElements.Num();
		OutElements.Add(MoveTemp(LocalElements[0]));
		OutElements.Add(MoveTemp(LocalElements[1]));
		return MinElem;
	}

	template<class OBJECT_ARRAY, class LEAF_TYPE, class T, int d>
	int32 TBoundingVolumeHierarchy<OBJECT_ARRAY, LEAF_TYPE, T, d>::GenerateNextLevel(const TVector<T, d>& GlobalMin, const TVector<T, d>& GlobalMax, const TArray<int32>& Objects, const int32 Level, TArray<TBVHNode<T, d>>& OutElements, TArray<LEAF_TYPE>& OutLeafs)
	{
		TArray<TBVHNode<T, d>> LocalElements;
		TArray<TArray<int32>> LocalObjects;
		TArray<LEAF_TYPE> LocalLeafs;
		LocalElements.SetNum(8);
		LocalObjects.SetNum(8);
		LocalLeafs.SetNum(8);
		TAABB<T, d> GlobalBox(GlobalMin, GlobalMax);
		const TVector<T, d> WorldCenter = GlobalBox.Center();
		for(int32 i = 0; i < Objects.Num(); ++i)
		{
			check(Objects[i] >= 0 && Objects[i] < GetObjectCount(*MObjects));
			const TAABB<T, d>& ObjectBox = Chaos::GetWorldSpaceBoundingBox(*MObjects, Objects[i], MWorldSpaceBoxes);
			const TVector<T, d> ObjectCenter = ObjectBox.Center();
			bool MinX = false, MaxX = false, MinY = false, MaxY = false, MinZ = false, MaxZ = false;
			if(ObjectBox.Min()[0] < WorldCenter[0])
			{
				MinX = true;
			}
			if(ObjectBox.Max()[0] >= WorldCenter[0])
			{
				MaxX = true;
			}
			if(ObjectBox.Min()[1] < WorldCenter[1])
			{
				MinY = true;
			}
			if(ObjectBox.Max()[1] >= WorldCenter[1])
			{
				MaxY = true;
			}
			if(ObjectBox.Min()[2] < WorldCenter[2])
			{
				MinZ = true;
			}
			if(ObjectBox.Max()[2] >= WorldCenter[2])
			{
				MaxZ = true;
			}
			check(MinX || MaxX);
			check(MinY || MaxY);
			check(MinZ || MaxZ);
			if(MinX && MinY && MinZ)
			{
				LocalObjects[0].Add(Objects[i]);
			}
			if(MaxX && MinY && MinZ)
			{
				LocalObjects[1].Add(Objects[i]);
			}
			if(MinX && MaxY && MinZ)
			{
				LocalObjects[2].Add(Objects[i]);
			}
			if(MaxX && MaxY && MinZ)
			{
				LocalObjects[3].Add(Objects[i]);
			}
			if(MinX && MinY && MaxZ)
			{
				LocalObjects[4].Add(Objects[i]);
			}
			if(MaxX && MinY && MaxZ)
			{
				LocalObjects[5].Add(Objects[i]);
			}
			if(MinX && MaxY && MaxZ)
			{
				LocalObjects[6].Add(Objects[i]);
			}
			if(MaxX && MaxY && MaxZ)
			{
				LocalObjects[7].Add(Objects[i]);
			}
		}
		// Build each child subtree into local arrays in parallel, then merge them deterministically below.
		// See the 2-way overload above: appending to the shared array under a lock made node ordering
		// thread-order-dependent and the cooked output non-deterministic.
		TArray<TBVHNode<T, d>> ChildElements[8];
		TArray<LEAF_TYPE> ChildLeafs[8];
		int32 ChildStartIndex[8] = { INDEX_NONE, INDEX_NONE, INDEX_NONE, INDEX_NONE, INDEX_NONE, INDEX_NONE, INDEX_NONE, INDEX_NONE };
		PhysicsParallelFor(8,
		[this, &LocalElements, &LocalObjects, &LocalLeafs, &ChildStartIndex, &ChildElements, &ChildLeafs, &Objects, Level, &GlobalBox, &WorldCenter]
		(int32 GroupIndex)
		{
			TVector<T, d> Min = GlobalBox.Min();
			TVector<T, d> Max = GlobalBox.Max();
			if(GroupIndex % 2 == 0)
			{
				Max[0] = WorldCenter[0];
			}
			else
			{
				Min[0] = WorldCenter[0];
			}
			if((GroupIndex / 2) % 2 == 0)
			{
				Max[1] = WorldCenter[1];
			}
			else
			{
				Min[1] = WorldCenter[1];
			}
			if(GroupIndex < 4)
			{
				Max[2] = WorldCenter[2];
			}
			else
			{
				Min[2] = WorldCenter[2];
			}
			LocalElements[GroupIndex].MMin = Min;
			LocalElements[GroupIndex].MMax = Max;
			LocalElements[GroupIndex].MAxis = -1;
			LocalElements[GroupIndex].LeafIndex = INDEX_NONE;
			if(LocalObjects[GroupIndex].Num() > MIN_NUM_OBJECTS &&
				Level < MMaxLevels &&
				LocalObjects[GroupIndex].Num() < Objects.Num())
			{
				TAABB<T, d> LocalBox(Min, Max);
				int32 NextAxis = 0;
				TVector<T, d> LocalExtents = LocalBox.Extents();
				if(LocalExtents[2] > LocalExtents[0] && LocalExtents[2] > LocalExtents[1])
				{
					NextAxis = 2;
				}
				else if(LocalExtents[1] > LocalExtents[0])
				{
					NextAxis = 1;
				}
				if(LocalExtents[NextAxis] < LocalExtents[(NextAxis + 1) % 3] * 1.25 &&
					LocalExtents[NextAxis] < LocalExtents[(NextAxis + 2) % 3] * 1.25 &&
					LocalObjects[GroupIndex].Num() > 4 * MIN_NUM_OBJECTS)
				{
					NextAxis = -1;
				}
				LocalElements[GroupIndex].MAxis = NextAxis;
				ChildStartIndex[GroupIndex] = GenerateNextLevel(
					LocalElements[GroupIndex].MMin,
					LocalElements[GroupIndex].MMax,
					LocalObjects[GroupIndex],
					NextAxis,
					Level + 1,
					true,
					ChildElements[GroupIndex],
					ChildLeafs[GroupIndex]);
			}
			else
			{
				LocalLeafs[GroupIndex] = MakeNewLeaf<OBJECT_ARRAY, T, d, TPayloadType>(
					*MObjects,
					LocalObjects[GroupIndex],
					static_cast<LEAF_TYPE*>(nullptr));
			}
		});

		for(int32 GroupIndex = 0; GroupIndex < 8; ++GroupIndex)
		{
			if(ChildStartIndex[GroupIndex] != INDEX_NONE)
			{
				const int32 ElemOffset = OutElements.Num();
				const int32 LeafOffset = OutLeafs.Num();
				for(TBVHNode<T, d>& ChildNode : ChildElements[GroupIndex])
				{
					if(ChildNode.MChildren.Num() > 0)
					{
						for(int32& GrandChildIndex : ChildNode.MChildren)
						{
							GrandChildIndex += ElemOffset;
						}
					}
					else
					{
						ChildNode.LeafIndex += LeafOffset;
					}
				}
				OutElements.Append(MoveTemp(ChildElements[GroupIndex]));
				OutLeafs.Append(MoveTemp(ChildLeafs[GroupIndex]));
				const int32 ChildRoot = ChildStartIndex[GroupIndex] + ElemOffset;
				for(int32 j = 0; j < (LocalElements[GroupIndex].MAxis == -1 ? 8 : 2); j++)
				{
					LocalElements[GroupIndex].MChildren.Add(ChildRoot + j);
				}
			}
			else
			{
				LocalElements[GroupIndex].LeafIndex = OutLeafs.Num();
				OutLeafs.Add(MoveTemp(LocalLeafs[GroupIndex]));
			}
			check(LocalElements[GroupIndex].LeafIndex < OutLeafs.Num() || LocalElements[GroupIndex].MChildren.Num() > 0);
		}
		const int32 MinElem = OutElements.Num();
		OutElements.Append(MoveTemp(LocalElements));
		return MinElem;
	}

	template <typename OBJECT_ARRAY, typename TPayloadType, typename T, int d>
	void FixupLeafObj(const OBJECT_ARRAY& Objects, TArray<TBoundingVolume<TPayloadType, T, d>>& Leafs)
	{
#if CHAOS_PARTICLEHANDLE_TODO
		for(TBoundingVolume<OBJECT_ARRAY, TPayloadType, T, d>& Leaf : Leafs)
		{
			Leaf.SetObjects(Objects);
		}
#endif
	}

	template <typename OBJECT_ARRAY>
	void FixupLeafObj(const OBJECT_ARRAY& Objects, TArray<TArray<int32>>& Leafs)
	{}

	template<class OBJECT_ARRAY, class LEAF_TYPE, class T, int d>
	void TBoundingVolumeHierarchy<OBJECT_ARRAY, LEAF_TYPE, T, d>::Serialize(FArchive& Ar)
	{
		Ar << MGlobalObjects;
		TBox<T, d>::SerializeAsAABBs(Ar, MWorldSpaceBoxes);
		Ar << MMaxLevels << Elements << Leafs;
		if(Ar.IsLoading())
		{
			FixupLeafObj(*MObjects, Leafs);
		}
	}
}

//template class CHAOS_API Chaos::TBoundingVolume<int32, FReal, 3>;
template class Chaos::TBoundingVolumeHierarchy<TArray<Chaos::FSphere*>, TArray<int32>, Chaos::FReal, 3>;
template class Chaos::TBoundingVolumeHierarchy<TArray<Chaos::TTetrahedron<Chaos::FReal>*>, TArray<int32>, Chaos::FReal, 3>;
template class Chaos::TBoundingVolumeHierarchy<Chaos::TPBDRigidParticles<Chaos::FReal, 3>, TArray<int32>, Chaos::FReal, 3>;
template class Chaos::TBoundingVolumeHierarchy<Chaos::FParticles, TArray<int32>, Chaos::FReal, 3>;
template class Chaos::TBoundingVolumeHierarchy<Chaos::TGeometryParticles<Chaos::FReal, 3>, TArray<int32>, Chaos::FReal, 3>;
template class Chaos::TBoundingVolumeHierarchy<Chaos::TPBDRigidParticles<Chaos::FReal, 3>, Chaos::TBoundingVolume<Chaos::TPBDRigidParticleHandle<Chaos::FReal, 3>*, Chaos::FReal, 3>, Chaos::FReal, 3>;
template class Chaos::TBoundingVolumeHierarchy<Chaos::TGeometryParticles<Chaos::FReal, 3>, Chaos::TBoundingVolume<Chaos::TGeometryParticleHandle<Chaos::FReal, 3>*, Chaos::FReal, 3>, Chaos::FReal, 3>;
template class Chaos::TBoundingVolumeHierarchy<TArray<Chaos::FImplicitObjectPtr>, TArray<int32>, Chaos::FReal, 3>;
template class Chaos::TBoundingVolumeHierarchy<TArray<Chaos::Private::FImplicitBVHObject>, TArray<int32>, Chaos::FReal, 3>;

