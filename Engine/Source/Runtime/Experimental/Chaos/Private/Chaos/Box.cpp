// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/Box.h"

//PRAGMA_DISABLE_OPTIMIZATION

namespace Chaos
{
	
	// Indices below must match TAABB::GetVertex (ascii art below was copied from there)
	// because TBox uses TAABB::SupportCore which returns a vertex index
	// NOTE: This shows a right-handed coordinate system, but we are left-handed...
	//
	//	   6---------7
	//	  /|        /|
	//	 / |       / |
	//	4---------5  |
	//	|  |      |  |
	//	|  2------|--3
	//	| /       | /
	//	|/        |/
	//	0---------1
	//
	//  Z
	//  |  Y
	//  | /
	//  |/
	//  o-----X
	// 
	//

	FConvexHalfEdgeStructureDataS16 MakeBoxStructureData()
	{
		// NOTE: planes vertices are counter-clockwise when looking at the face from outside 
		// the box in the diagram above. Plane order defined by SNormals.
		TArray<TArray<int32>> PlaneVertices
		{
			{ 0, 4, 6, 2 },	// -X,
			{ 0, 1, 5, 4 },	// -Y
			{ 0, 2, 3, 1 },	// -Z
			{ 1, 3, 7, 5 },	//  X
			{ 2, 6, 7, 3 },	//  Y
			{ 4, 5, 7, 6 },	//  Z
		};

		return FConvexHalfEdgeStructureDataS16::MakePlaneVertices(PlaneVertices, 8);
	}

	// NOTE: Plane order must match order implied by TAABB::GetIndex/GetVertex
	// i.e., negative pointing normals first
	template<typename T, int D> TArray<FVec3> TBox<T, D>::SNormals =
	{
		FVec3(-1, 0, 0),	// -X
		FVec3( 0,-1, 0),	// -Y
		FVec3( 0, 0,-1),	// -Z
		FVec3( 1, 0, 0),	//  X
		FVec3( 0, 1, 0),	//  Y
		FVec3( 0, 0, 1),	//  Z
	};
	
	// NOTE: Vertex order must match order implied by TAABB::GetIndex/GetVertex
	template<typename T, int D> TArray<FVec3> TBox<T, D>::SVertices =
	{
		FVec3(-1,-1,-1),	// 0 
		FVec3( 1,-1,-1),	// 1 
		FVec3(-1, 1,-1),	// 2 
		FVec3( 1, 1,-1),	// 3 
		FVec3(-1,-1, 1),	// 4 
		FVec3( 1,-1, 1),	// 5 
		FVec3(-1, 1, 1),	// 6 
		FVec3( 1, 1, 1),	// 7 
	};

	template<typename T, int D> FConvexHalfEdgeStructureDataS16 TBox<T, D>::SStructureData = MakeBoxStructureData();

	template class TBox<FReal, 3>;
}