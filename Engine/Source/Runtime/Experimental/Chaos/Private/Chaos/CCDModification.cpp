// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/CCDModification.h"
#include "Chaos/CCDUtilities.h"
#include "Chaos/Collision/PBDCollisionConstraint.h"
#include "Chaos/Collision/ParticlePairMidPhase.h"
#include "Chaos/Particle/ParticleUtilities.h"
#include "Chaos/PBDCollisionConstraints.h"

namespace Chaos
{
	bool FCCDModifier::IsSweepHit() const
	{
		if (Constraint != nullptr)
		{
			return Constraint->GetCCDSweepEnabled() && (Constraint->NumManifoldPoints() > 0);
		}
		return false;
	}

	FReal FCCDModifier::GetSweepHitTOI() const
	{
		if (Constraint != nullptr)
		{
			return Constraint->GetCCDTimeOfImpact();
		}
		return FReal(1);
	}

	FVec3 FCCDModifier::GetWorldSweepHitLocation(const int32 ParticleIndex) const
	{
		check(ParticleIndex >= 0);
		check(ParticleIndex < 2);

		if (Constraint != nullptr)
		{
			// NOTE: sweeps only generate 1 manifold point
			if (Constraint->NumManifoldPoints() > 0)
			{
				const FManifoldPoint& ManifoldPoint = Constraint->GetManifoldPoint(0);
				const FRigidTransform3 ParticleWorldTransform = FCCDHelpers::GetParticleTransformAtTOI(GetParticle(ParticleIndex), GetSweepHitTOI(), Accessor->GetDt());
				const FRigidTransform3 ShapeWorldTransform = Constraint->GetShapeRelativeTransform(ParticleIndex) * ParticleWorldTransform;
				return ShapeWorldTransform.TransformPositionNoScale(FVec3(ManifoldPoint.ContactPoint.ShapeContactPoints[ParticleIndex]));
			}
		}
		return FVec3(0);
	}

	FVec3 FCCDModifier::GetWorldSweepHitNormal() const
	{
		if (Constraint != nullptr)
		{
			// NOTE: sweeps only generate 1 manifold point
			if (Constraint->NumManifoldPoints() > 0)
			{
				const FManifoldPoint& ManifoldPoint = Constraint->GetManifoldPoint(0);
				const FRigidTransform3& ShapeWorldTransform = Constraint->GetShapeWorldTransform1();
				return ShapeWorldTransform.TransformVectorNoScale(FVec3(ManifoldPoint.ContactPoint.ShapeContactNormal));
			}
		}
		return FVec3(0,0,1);
	}

	const FGeometryParticleHandle* FCCDModifier::GetParticle(const int32 ParticleIndex) const
	{
		check(ParticleIndex >= 0);
		check(ParticleIndex < 2);

		if (Constraint != nullptr)
		{
			if (ParticleIndex == 0)
			{
				return Constraint->GetParticle0();
			}
			else
			{
				return Constraint->GetParticle1();
			}
		}
		return nullptr;
	}

	const FGeometryParticleHandle* FCCDModifier::GetOtherParticle(const FGeometryParticleHandle* InParticle) const
	{
		if (Constraint != nullptr)
		{
			const FGeometryParticleHandle* Particle0 = Constraint->GetParticle0();
			const FGeometryParticleHandle* Particle1 = Constraint->GetParticle1();
			if (InParticle == Particle0)
			{
				return Particle1;
			}
			else if (InParticle == Particle1)
			{
				return Particle0;
			}
		}
		return nullptr;
	}

	void FCCDModifier::Enable()
	{
		if (Constraint != nullptr)
		{
			Constraint->SetModifierApplied();
			Constraint->SetDisabled(false);
		}
	}

	void FCCDModifier::Disable()
	{
		if (Constraint != nullptr)
		{
			Constraint->SetModifierApplied();
			Constraint->SetDisabled(true);
		}
	}

	void FCCDModifier::ConvertToProbe()
	{
		if (Constraint != nullptr)
		{
			Constraint->SetModifierApplied();
			Constraint->SetIsProbe(true);
		}
	}

	//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	FCCDModifierParticleIterator& FCCDModifierParticleIterator::MoveToBegin()
	{ 
		ConstraintIndex = -1;
		MoveToNext();
		return *this;
	}
	
	FCCDModifierParticleIterator& FCCDModifierParticleIterator::MoveToEnd()
	{ 
		ConstraintIndex = Range->GetNumConstraints();
		PairModifier.Reset();
		return *this;
	}
	
	FCCDModifierParticleIterator& FCCDModifierParticleIterator::MoveToNext()
	{
		check(ConstraintIndex < Range->GetNumConstraints());

		++ConstraintIndex;
		if (ConstraintIndex < Range->GetNumConstraints())
		{
			PairModifier = FCCDModifier(Range->GetAccessor(), Range->GetConstraint(ConstraintIndex));
		}
		else
		{
			PairModifier.Reset();
		}
		return *this;
	}

	//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	FCCDModifierParticleRange::FCCDModifierParticleRange(
		FCCDModifierAccessor* InAccessor, 
		FGeometryParticleHandle* InParticle)
		: Accessor(InAccessor)
		, Particle(InParticle)
	{
		CollectConstraints();
	}

	FCCDModifierParticleIterator FCCDModifierParticleRange::begin()
	{
		return FCCDModifierParticleIterator(this).MoveToBegin();
	}

	FCCDModifierParticleIterator FCCDModifierParticleRange::end()
	{
		return FCCDModifierParticleIterator(this).MoveToEnd();
	}

	void FCCDModifierParticleRange::CollectConstraints()
	{
		Constraints.Reset();

		Particle->ParticleCollisions().VisitCollisions(
			[this](FPBDCollisionConstraint& Constraint) -> ECollisionVisitorResult
			{
				if (Constraint.GetCCDEnabled())
				{
					Constraints.Add(&Constraint);
				}
				return ECollisionVisitorResult::Continue;
			});
	}

	//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	FCCDModifierAccessor::FCCDModifierAccessor(const FReal InDt)
		: Dt(InDt)
	{
	}

	FCCDModifierParticleRange FCCDModifierAccessor::GetModifiers(FGeometryParticleHandle* Particle)
	{
		return FCCDModifierParticleRange(this, Particle);
	}
}
