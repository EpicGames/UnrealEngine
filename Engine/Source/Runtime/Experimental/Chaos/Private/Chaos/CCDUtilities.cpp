// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/CCDUtilities.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/PBDCollisionConstraints.h"
#include "Chaos/PBDRigidsEvolutionGBF.h"
#include "Chaos/CollisionResolution.h"
#include "Chaos/ChaosDebugDraw.h"
#include "Chaos/DebugDrawQueue.h"

//UE_DISABLE_OPTIMIZATION

namespace Chaos
{
	namespace CVars
	{
		bool bClearNullCCDConstraintsOnApply = true;
		FAutoConsoleVariableRef CVarClearNullCCDConstraintsOnApply(TEXT("p.Chaos.Collision.CCD.ClearNullConstraints"), bClearNullCCDConstraintsOnApply, TEXT("Remove null constraints encountered during correction application for CCD."));

		// @todo(chaos): most of these cvars should be settings per particle

		// NOTE: With this disabled secondary CCD collisions will often be missed
		// @todo(chaos): resweeping also change contacts so it raises questions about collision modifier callbacks and CCD
		bool bChaosCollisionCCDEnableResweep = true;
		FAutoConsoleVariableRef CVarChaosCollisionCCDEnableResweep(TEXT("p.Chaos.Collision.CCD.EnableResweep"), bChaosCollisionCCDEnableResweep, TEXT("Enable resweep for CCD. Resweeping allows CCD to catch more secondary collisions but also is more costly. Default is true."));

		// NOTE: With this disabled, secondary collisions can be missed. When enabled, velocity will not be visually consistent after CCD collisions (if ChaosCollisionCCDConstraintMaxProcessCount is too low)
		bool bChaosCollisionCCDAllowClipping = true;
		FAutoConsoleVariableRef CVarChaosCollisionCCDAllowClipping(TEXT("p.Chaos.Collision.CCD.AllowClipping"), bChaosCollisionCCDAllowClipping, TEXT("This will clip the CCD object at colliding positions when computation budgets run out. Default is true. Turning this option off might cause tunneling."));

		// By default, we stop processing CCD contacts after a single CCD interaction
		// This will result in a visual velocity glitch when it happens, but usually this doesn't matter since the impact is very high energy anyway
		int32 ChaosCollisionCCDConstraintMaxProcessCount = 1;
		FAutoConsoleVariableRef CVarChaosCollisionCCDConstraintMaxProcessCount(TEXT("p.Chaos.Collision.CCD.ConstraintMaxProcessCount"), ChaosCollisionCCDConstraintMaxProcessCount, TEXT("The max number of times each constraint can be resolved when applying CCD constraints. Default is 2. The larger this number is, the more fully CCD constraints are resolved."));

		// Determines when CCD is switched on
		FRealSingle CCDEnableThresholdBoundsScale = 0.4f;
		FAutoConsoleVariableRef  CVarCCDEnableThresholdBoundsScale(TEXT("p.Chaos.CCD.EnableThresholdBoundsScale"), CCDEnableThresholdBoundsScale , TEXT("CCD is used when object position is changing > smallest bound's extent * BoundsScale. 0 will always Use CCD. Values < 0 disables CCD."));

		// Determines how much penetration CCD leaves after rewinding to TOI. Must be less than CCDEnableThresholdBoundsScale
		Chaos::FRealSingle CCDAllowedDepthBoundsScale = 0.2f;
		FAutoConsoleVariableRef CVarCCDAllowedDepthBoundsScale(TEXT("p.Chaos.CCD.AllowedDepthBoundsScale"), CCDAllowedDepthBoundsScale, TEXT("When rolling back to TOI, allow (smallest bound's extent) * AllowedDepthBoundsScale, instead of rolling back to exact TOI w/ penetration = 0."));

		// When enabled, CCD TOI is set to the first contact which gets a depth of (CCDAllowedDepthBoundsScale * Object Size).
		// When disabled, CCD will find the first contact, ignoreing the rest, and then fix the TOI to result in the specified depth.
		// This fixes the issue where (when disabled) we would set TOI to some value that would leave us penetrating another (later) object. This
		// could happen when colliding with the floor near the base of a ramp for example.
		bool bCCDNewTargetDepthMode = true;
		FAutoConsoleVariableRef  CVarCCDNewTargetDepthMode(TEXT("p.Chaos.CCD.NewTargetDepthMode"), bCCDNewTargetDepthMode, TEXT("Find the first contact with that results in a penetration of (CCDAllowedDepthBoundsScale*Size) as opposed to the first contact"));

		int32 CCDAxisThresholdMode = 2;
		FAutoConsoleVariableRef  CVarCCDAxisThresholdMode(TEXT("p.Chaos.CCD.AxisThresholdMode"), CCDAxisThresholdMode , TEXT("Change the mode used to generate CCD axis threshold bounds for particle geometries.\n0: Use object bounds\n1: Find the thinnest object bound on any axis and use it for all CCD axes\n2: On each axis, use the thinnest shape bound on that axis\n3: Find the thinnest shape bound on any axis and use this for all axes"));

		bool bCCDAxisThresholdUsesProbeShapes = false;
		FAutoConsoleVariableRef  CVarCCDAxisThresholdUsesProbeShapes(TEXT("p.Chaos.CCD.CCDAxisThresholdUsesProbeShapes"), bCCDAxisThresholdUsesProbeShapes , TEXT("When true, probe shapes are considered for CCD axis threshold computation, and can generate contacts in the initial CCD phase."));

		bool bCCDSweepsUseProbeShapes = false;
		FAutoConsoleVariableRef  CVarCCDSweepsUseProbeShapes(TEXT("p.Chaos.CCD.CCDSweepsUseProbeShapes"), bCCDSweepsUseProbeShapes , TEXT("When true, probe shapes can be swept for more accurate collision detection."));

		// How many post-solve CCD correction iterations to run
		int32 ChaosCollisionCCDCorrectionIterations = 4;
		FAutoConsoleVariableRef CVarChaosCollisionCCDCorrectionIterations(TEXT("p.Chaos.Collision.CCD.CorrectionIterations"), ChaosCollisionCCDCorrectionIterations, TEXT("The number of post-solve CCD correction ietaryions to run."));

		// A multiplier on the constraint CCD threshold that determines how much penetration we allow in the correction phase
		FRealSingle ChaosCollisionCCDCorrectionPhiToleranceScale = 0.02f;
		FAutoConsoleVariableRef CVarChaosCollisionCCDCorrectionPhiToleranceScale(TEXT("p.Chaos.Collision.CCD.CorrectionPhiToleranceScale"), ChaosCollisionCCDCorrectionPhiToleranceScale, TEXT("How much penetration we allow during the correction phase (multiplier on shape size)"));

		extern int32 ChaosSolverDrawCCDInteractions;

#if CHAOS_DEBUG_DRAW
		extern DebugDraw::FChaosDebugDrawSettings ChaosSolverDebugDebugDrawSettings;	
#endif
	}

	void FCCDParticle::AddOverlappingDynamicParticle(FCCDParticle* const InParticle)
	{
		OverlappingDynamicParticles.Add(InParticle);
	}

	void FCCDParticle::AddConstraint(FCCDConstraint* const Constraint)
	{
		AttachedCCDConstraints.Add(Constraint);
	}

	int32 FCCDConstraint::GetFastMovingKinematicIndex(const FPBDCollisionConstraint* Constraint, const FVec3 Displacements[]) const
	{
		for (int32 i = 0; i < 2; i++)
		{
			const TPBDRigidParticleHandle<FReal, 3>* Rigid = Constraint->GetParticle(i)->CastToRigidParticle();
			if (Rigid && Rigid->ObjectState() == EObjectStateType::Kinematic)
			{
				if (FCCDHelpers::DeltaExceedsThreshold(Rigid->CCDAxisThreshold(), Displacements[i], Rigid->GetQ()))
				{
					return i;
				}
			}
		}
		return INDEX_NONE;
	}

	void FCCDManager::ApplyCCD(const FReal Dt, TArrayView<FPBDCollisionConstraint* const> SweptConstraints, const int32 NumDynamicParticles)
	{
		if (SweptConstraints.Num() > 0)
		{
			ApplySweptConstraints(Dt, SweptConstraints, NumDynamicParticles);
			UpdateSweptConstraints(Dt);
			OverwriteXUsingV(Dt);
		}
	}

	void FCCDManager::ApplySweptConstraints(const FReal Dt, TArrayView<FPBDCollisionConstraint* const> SweptConstraints, const int32 NumDynamicParticles)
	{
		const bool bNeedCCDSolve = Init(Dt, SweptConstraints, NumDynamicParticles);
		if (!bNeedCCDSolve)
		{
			return;
		}

		// Use simpler processing loop if we have only 1 CCD iteration
		const bool bUseSimpleCCD = (CVars::ChaosCollisionCCDConstraintMaxProcessCount == 1);

		AssignParticleIslandsAndGroupParticles();
		AssignConstraintIslandsAndRecordConstraintNum();
		GroupConstraintsWithIslands();
		PhysicsParallelFor(IslandNum, [&](const int32 Island)
		{
			if (bUseSimpleCCD)
			{
				ApplyIslandSweptConstraints2(Island, Dt);

			}
			else
			{
				ApplyIslandSweptConstraints(Island, Dt);
			}
		});
	}

	bool FCCDManager::Init(const FReal Dt, TArrayView<FPBDCollisionConstraint* const> SweptConstraints, const int32 NumDynamicParticles)
	{
		// We store pointers to CCDParticle in CCDConstraint and GroupedCCDParticles, therefore we need to make sure to reserve enough space 
		// for TArray CCDParticles so that reallocation does not happen in the for loop. If reallocation happens, some pointers may become 
		// invalid and this could cause serious bugs. We know that the number of CCDParticles cannot exceed SweptConstraints.Num() * 2 or NumDynamicParticles. 
		// Therefore this makes sure reallocation does not happen.
		CCDParticles.Reset();
		CCDParticles.Reserve(FMath::Min(SweptConstraints.Num() * 2, NumDynamicParticles));
		ParticleToCCDParticle.Reset();
		CCDConstraints.Reset();
		CCDConstraints.Reserve(SweptConstraints.Num());
		bool bNeedCCDSolve = false;
		for (FPBDCollisionConstraint* Constraint : SweptConstraints)
		{
			// A contact can be disabled by a user callback or contact pruning so we need to ignore these.
			// NOTE: It important that we explicitly check for disabled here, rather than for 0 manifold points since we
			// may want to use the contact later if resweeping is enabled. 
			if (!Constraint->IsEnabled())
			{
				continue;
			}

			// If we don't need to sweep for this constraint, skip it
			if (!Constraint->GetCCDSweepEnabled())
			{
				continue;
			}

			// For now, probe constraints do not work with CCD
			if (Constraint->IsProbe())
			{
				continue;
			}

			// Create CCDParticle for all dynamic particles affected by swept constraints (UseCCD() could be either true or false). 
			// For static or kinematic particles, this pointer remains to be nullptr.
			FCCDParticle* CCDParticlePair[2] = {nullptr, nullptr};
			bool IsDynamic[2] = {false, false};
			FVec3 Displacements[2] = {FVec3(0.f), FVec3(0.f)};
			for (int i = 0; i < 2; i++)
			{
				TPBDRigidParticleHandle<FReal, 3>* RigidParticle = Constraint->GetParticle(i)->CastToRigidParticle();
				FCCDParticle* CCDParticle = nullptr;
				const bool IsParticleDynamic = RigidParticle && RigidParticle->ObjectState() == EObjectStateType::Dynamic;
				if (IsParticleDynamic)
				{
					FCCDParticle** FoundCCDParticle = ParticleToCCDParticle.Find(RigidParticle);
					if (!FoundCCDParticle)
					{
						// See comments above - we cannot allow CCDParticles to resize
						// @todo(chaos): clean this up - use indices, chunked array, or something
						check(CCDParticles.Num() < CCDParticles.Max());
						CCDParticles.Add(FCCDParticle(RigidParticle));
						CCDParticle = &CCDParticles.Last();
						ParticleToCCDParticle.Add(RigidParticle, CCDParticle);
					}
					else
					{
						CCDParticle = *FoundCCDParticle;
					}
					IsDynamic[i] = IsParticleDynamic;
				}
				CCDParticlePair[i] = CCDParticle; 
				IsDynamic[i] = IsParticleDynamic;

				if (RigidParticle)
				{
					// One can also use P - X for dynamic particles. But notice that for kinematic particles, 
					// both P and X are end-frame positions and P - X won't work for kinematic particles.
					Displacements[i] = RigidParticle->GetV() * Dt;
				}
			}

			// Determine if this particle pair should trigger CCD
			const auto Particle0 = Constraint->GetParticle(0);
			const auto Particle1 = Constraint->GetParticle(1);
			bNeedCCDSolve = FCCDHelpers::DeltaExceedsThreshold(*Particle0, *Particle1, Dt);

			// make sure we ignore pairs that don't include any dynamics
			if (CCDParticlePair[0] != nullptr || CCDParticlePair[1] != nullptr)
			{
				CCDConstraints.Add(FCCDConstraint(Constraint, CCDParticlePair, Displacements));
				for (int32 i = 0; i < 2; i++)
				{
					if (CCDParticlePair[i] != nullptr)
					{
						CCDParticlePair[i]->AddConstraint(&CCDConstraints.Last());
					}
				}

				if (IsDynamic[0] && IsDynamic[1])
				{
					CCDParticlePair[0]->AddOverlappingDynamicParticle(CCDParticlePair[1]);
					CCDParticlePair[1]->AddOverlappingDynamicParticle(CCDParticlePair[0]);
				}
			}
		}
		return bNeedCCDSolve;
	}

	void FCCDManager::AssignParticleIslandsAndGroupParticles()
	{
		// Use DFS to find connected dynamic particles and assign islands for them.
		// In the mean time, record numbers in IslandParticleStart and IslandParticleNum
		// Group particles into GroupedCCDParticles based on islands.
		IslandNum = 0;
		IslandStack.Reset();
		GroupedCCDParticles.Reset();
		IslandParticleStart.Reset();
		IslandParticleNum.Reset();
		for (FCCDParticle &CCDParticle : CCDParticles)
		{
			if (CCDParticle.Island != INDEX_NONE || CCDParticle.Particle->ObjectState() != EObjectStateType::Dynamic)
			{
				continue;
			}
			FCCDParticle* CurrentParticle = &CCDParticle;
			CurrentParticle->Island = IslandNum;
			IslandStack.Push(CurrentParticle);
			IslandParticleStart.Push(GroupedCCDParticles.Num());
			int32 CurrentIslandParticleNum = 0;
			while (IslandStack.Num() > 0)
			{
				CurrentParticle = IslandStack.Pop();
				GroupedCCDParticles.Push(CurrentParticle);
				CurrentIslandParticleNum++;
				for (FCCDParticle* OverlappingParticle : CurrentParticle->OverlappingDynamicParticles)
				{
					if (OverlappingParticle->Island == INDEX_NONE)
					{
						OverlappingParticle->Island = IslandNum;
						IslandStack.Push(OverlappingParticle);
					}
				}
			}
			IslandParticleNum.Push(CurrentIslandParticleNum);
			IslandNum++;
		}
	}

	void FCCDManager::AssignConstraintIslandsAndRecordConstraintNum()
	{
		// Assign island to constraints based on particle islands
		// In the mean time, record IslandConstraintNum
		IslandConstraintNum.SetNum(IslandNum);
		for (int32 i = 0; i < IslandNum; i++)
		{
			IslandConstraintNum[i] = 0;
		}

		for (FCCDConstraint &CCDConstraint : CCDConstraints)
		{
			int32 Island = INDEX_NONE;
			if (CCDConstraint.Particle[0])
			{
				Island = CCDConstraint.Particle[0]->Island;
			}
			if (Island == INDEX_NONE)
			{
				// non-dynamic pairs are already ignored in Init() so if Particle 0 is null the second one should not be 
				ensure(CCDConstraint.Particle[1] != nullptr);

				if (CCDConstraint.Particle[1])
				{
					Island = CCDConstraint.Particle[1]->Island;
				}	
			}
			CCDConstraint.Island = Island;
			IslandConstraintNum[Island]++;
		}
	}

	void FCCDManager::GroupConstraintsWithIslands()
	{
		// Group constraints based on island
		// In the mean time, record IslandConstraintStart, IslandConstraintEnd
		IslandConstraintStart.SetNum(IslandNum + 1);
		IslandConstraintEnd.SetNum(IslandNum);
		IslandConstraintStart[0] = 0;
		for (int32 i = 0; i < IslandNum; i++)
		{
			IslandConstraintEnd[i] = IslandConstraintStart[i];
			IslandConstraintStart[i + 1] = IslandConstraintStart[i] + IslandConstraintNum[i];
		}

		SortedCCDConstraints.SetNum(CCDConstraints.Num());
		for (FCCDConstraint &CCDConstraint : CCDConstraints)
		{
			const int32 Island = CCDConstraint.Island;
			SortedCCDConstraints[IslandConstraintEnd[Island]] = &CCDConstraint;
			IslandConstraintEnd[Island]++;
		}
	}

	bool CCDConstraintSortPredicate(const FCCDConstraint* Constraint0, const FCCDConstraint* Constraint1)
	{
		return Constraint0->SweptConstraint->GetCCDTimeOfImpact() < Constraint1->SweptConstraint->GetCCDTimeOfImpact();
	}

	// Differences to the original ApplyIslandSweptConstraints
	// - only 1 CCD iteration, and we roll back positions to the first contact on each body pair
	// - if a body pair has multiple constraints (multiple shapes) only the first contact need to processed
	// - no impulses are applied by CCD. NOTE: this only works if the TOI rollback leaves some penetration
	//
	void FCCDManager::ApplyIslandSweptConstraints2(const int32 Island, const FReal Dt)
	{
		const int32 ConstraintStart = IslandConstraintStart[Island];
		const int32 ConstraintNum = IslandConstraintNum[Island];
		const int32 ConstraintEnd = IslandConstraintEnd[Island];
		const int32 ParticleStart = IslandParticleStart[Island];
		const int32 ParticleNum = IslandParticleNum[Island];

		if (ConstraintNum == 0)
		{
			return;
		}

		// We assume that the particle's center of mass moved in a straight line since the previous tick.
		// Modify X so that X-to-P is a straight line so that we can interpolate between X and P using the TOI
		// to get the position at that time. This is required to handle objects with a CoM offset from the actor position.
		// We will undo this manipulation at the end.
		// NOTE: We do not modify the previous rotation R here - we just use the current rotation Q everywhere
		// @todo(chaos): we should store the sweep positions in CCDParticle and only modify the original particle at the end
		for (int32 i = ParticleStart; i < ParticleStart + ParticleNum; i++)
		{
			GroupedCCDParticles[i]->Particle->SetX(GroupedCCDParticles[i]->Particle->GetP() - GroupedCCDParticles[i]->Particle->GetV() * Dt);
		}

		// Sort constraints based on TOI
		FReal IslandTOI = 0.f;
		ResetIslandParticles(Island);
		ResetIslandConstraints(Island);

		bool bSortRequired = true;
		int32 ConstraintIndex = ConstraintStart;
		while (ConstraintIndex < ConstraintEnd)
		{
			// If we updated some constraints, we need to sort so that we handle the next TOI event
			if (bSortRequired)
			{
				std::sort(SortedCCDConstraints.GetData() + ConstraintIndex, SortedCCDConstraints.GetData() + ConstraintStart + ConstraintNum, CCDConstraintSortPredicate);
				bSortRequired = false;
			}

			FCCDConstraint* CCDConstraint = SortedCCDConstraints[ConstraintIndex];
			FCCDParticle* CCDParticle0 = CCDConstraint->Particle[0];
			FCCDParticle* CCDParticle1 = CCDConstraint->Particle[1];

			IslandTOI = CCDConstraint->SweptConstraint->GetCCDTimeOfImpact();

			// Constraints whose TOIs are in the range of [0, 1) are resolved for this frame. TOI = 1 means that the two 
			// particles just start touching at the end of the frame and therefore cannot have tunneling this frame. 
			// So this TOI = 1 can be left to normal collisions or CCD in next frame.
			if (IslandTOI > 1)
			{
				break;
			}

			// If both particles are marked Done continue
			const bool bParticle0Done = ((CCDParticle0 == nullptr) || CCDParticle0->Done);
			const bool bParticle1Done = ((CCDParticle1 == nullptr) || CCDParticle1->Done);
			if (bParticle0Done && bParticle1Done)
			{
				ConstraintIndex++;
				continue;
			}

			CCDConstraint->ProcessedCount++;

			// In UpdateConstraintSwept, InitManifoldPoint requires P, Q to be at TOI=1., but the input of 
			// UpdateConstraintSwept requires transforms at current TOI. So instead of rewinding P, Q, we 
			// advance X, R to current TOI and keep P, Q at TOI=1.
			if (CCDParticle0 && !CCDParticle0->Done)
			{
				AdvanceParticleXToTOI(CCDParticle0, IslandTOI, Dt);
			}
			if (CCDParticle1 && !CCDParticle1->Done)
			{
				AdvanceParticleXToTOI(CCDParticle1, IslandTOI, Dt);
			}

#if CHAOS_DEBUG_DRAW
			// Debugdraw the shape at the current TOI
			if (CVars::ChaosSolverDrawCCDInteractions)
			{
				const DebugDraw::FChaosDebugDrawSettings& DebugDrawSettings = CVars::ChaosSolverDebugDebugDrawSettings;
				for (int32 ManifoldPointIndex = 0; ManifoldPointIndex < CCDConstraint->SweptConstraint->NumManifoldPoints(); ++ManifoldPointIndex)
				{
					const FManifoldPoint& ManifoldPoint = CCDConstraint->SweptConstraint->GetManifoldPoint(ManifoldPointIndex);
					if (ManifoldPoint.Flags.bDisabled)
					{
						continue;
					}
					const FVec3 ContactPos = CCDConstraint->SweptConstraint->GetShapeWorldTransform1().TransformPositionNoScale(FVec3(ManifoldPoint.ContactPoint.ShapeContactPoints[1]));
					const FVec3 ContactNormal = CCDConstraint->SweptConstraint->GetShapeWorldTransform1().TransformVectorNoScale(FVec3(ManifoldPoint.ContactPoint.ShapeContactNormal));
					FDebugDrawQueue::GetInstance().DrawDebugLine(ContactPos, ContactPos + DebugDrawSettings.DrawScale * 50 * ContactNormal, FColor::Red, false, UE_KINDA_SMALL_NUMBER, uint8(DebugDrawSettings.DrawPriority), DebugDrawSettings.LineThickness);
				}
			}
#endif

			// We don't normally apply impulses in the CCD phase because we leave on overlap for the main solver to handle. However if the
			// settings are such that this won't happen, we must apply the impulse here. CCD impulses assume point masses and no rotations or friction are applied.
			const bool bApplyImpulse = (CVars::CCDAllowedDepthBoundsScale <= 0);
			if (bApplyImpulse)
			{
				ApplyImpulse(CCDConstraint);
			}

			if (CCDParticle0)
			{
				if (CCDConstraint->FastMovingKinematicIndex != INDEX_NONE)
				{
					// @todo(chaos): can this just get the particle from the CCD constraint?
					const FConstGenericParticleHandle KinematicParticle = FGenericParticleHandle(CCDConstraint->SweptConstraint->GetParticle(CCDConstraint->FastMovingKinematicIndex));
					const FVec3 Normal = CCDConstraint->SweptConstraint->CalculateWorldContactNormal();
					const FVec3 Offset = FVec3::DotProduct(KinematicParticle->V() * ((1.f - IslandTOI) * Dt), Normal) * Normal;
					ClipParticleP(CCDParticle0, Offset);
				}
				else
				{
					ClipParticleP(CCDParticle0);
				}
				CCDParticle0->Done = true;
			}
			if (CCDParticle1)
			{
				if (CCDConstraint->FastMovingKinematicIndex != INDEX_NONE)
				{
					// @todo(chaos): can this just get the particle from the CCD constraint?
					const FConstGenericParticleHandle KinematicParticle = FGenericParticleHandle(CCDConstraint->SweptConstraint->GetParticle(CCDConstraint->FastMovingKinematicIndex));
					const FVec3 Normal = CCDConstraint->SweptConstraint->CalculateWorldContactNormal();
					const FVec3 Offset = FVec3::DotProduct(KinematicParticle->V() * ((1.f - IslandTOI) * Dt), Normal) * Normal;
					ClipParticleP(CCDParticle1, Offset);
				}
				else
				{
					ClipParticleP(CCDParticle1);
				}
				CCDParticle1->Done = true;
			}

			// We applied a CCD impulse and updated the particle positions, so we need to update all the constraints involving these particles
			if (CCDParticle0)
			{
				bSortRequired |= UpdateParticleSweptConstraints(CCDParticle0, IslandTOI, Dt);
			}
			if (CCDParticle1)
			{
				bSortRequired |= UpdateParticleSweptConstraints(CCDParticle1, IslandTOI, Dt);
			}

			ConstraintIndex++;
		}

#if CHAOS_DEBUG_DRAW
		// Debugdraw the shapes at the final position
		if (CVars::ChaosSolverDrawCCDInteractions)
		{
			for (int32 ParticleIndex = ParticleStart; ParticleIndex < ParticleStart + ParticleNum; ParticleIndex++)
			{
				const FGeometryParticleHandle* Particle = GroupedCCDParticles[ParticleIndex]->Particle;
				DebugDraw::DrawParticleShapes(FRigidTransform3::Identity, Particle, FColor::Green, &CVars::ChaosSolverDebugDebugDrawSettings);
			}
		}
#endif

		// SetX so that the implicit velocity and angular velocity will be calculated correctly in the solver step
		// NOTE: This is not the same as its original X if we have been rewound to a TOI.
		for (int32 i = ParticleStart; i < ParticleStart + ParticleNum; i++)
		{
			const FVec3 CoMPrev = GroupedCCDParticles[i]->Particle->PCom() - GroupedCCDParticles[i]->Particle->GetV() * Dt;
			const FVec3 CoMOffsetPrev = GroupedCCDParticles[i]->Particle->GetR() * GroupedCCDParticles[i]->Particle->CenterOfMass();
			GroupedCCDParticles[i]->Particle->SetX(CoMPrev - CoMOffsetPrev);
		}
	}

	void FCCDManager::ApplyIslandSweptConstraints(const int32 Island, const FReal Dt)
	{
		const int32 ConstraintStart = IslandConstraintStart[Island];
		const int32 ConstraintNum = IslandConstraintNum[Island];
		const int32 ConstraintEnd = IslandConstraintEnd[Island];
		check(ConstraintNum > 0);

		// Sort constraints based on TOI
		std::sort(SortedCCDConstraints.GetData() + ConstraintStart, SortedCCDConstraints.GetData() + ConstraintStart + ConstraintNum, CCDConstraintSortPredicate);
		FReal IslandTOI = 0.f;
		ResetIslandParticles(Island);
		ResetIslandConstraints(Island);
		int32 ConstraintIndex = ConstraintStart;
		while (ConstraintIndex < ConstraintEnd) 
		{
			FCCDConstraint *CCDConstraint = SortedCCDConstraints[ConstraintIndex];
			FCCDParticle* CCDParticle0 = CCDConstraint->Particle[0];
			FCCDParticle* CCDParticle1 = CCDConstraint->Particle[1];

			IslandTOI = CCDConstraint->SweptConstraint->GetCCDTimeOfImpact();

			// Constraints whose TOIs are in the range of [0, 1) are resolved for this frame. TOI = 1 means that the two 
			// particles just start touching at the end of the frame and therefore cannot have tunneling this frame. 
			// So this TOI = 1 can be left to normal collisions or CCD in next frame.
			if (IslandTOI > 1) 
			{
				break;
			}

			// If both particles are marked Done (due to clipping), continue
			if (CVars::bChaosCollisionCCDAllowClipping && (!CCDParticle0 || CCDParticle0->Done) && (!CCDParticle1 || CCDParticle1->Done))
			{
				ConstraintIndex++;
				continue;
			}
			
			ensure(CCDConstraint->ProcessedCount < CVars::ChaosCollisionCCDConstraintMaxProcessCount);

			// In UpdateConstraintSwept, InitManifoldPoint requires P, Q to be at TOI=1., but the input of 
			// UpdateConstraintSwept requires transforms at current TOI. So instead of rewinding P, Q, we 
			// advance X, R to current TOI and keep P, Q at TOI=1.
			if (CCDParticle0 && !CCDParticle0->Done)
			{
				AdvanceParticleXToTOI(CCDParticle0, IslandTOI, Dt);
			}
			if (CCDParticle1 && !CCDParticle1->Done)
			{
				AdvanceParticleXToTOI(CCDParticle1, IslandTOI, Dt);
			}

			ApplyImpulse(CCDConstraint);
			CCDConstraint->ProcessedCount++;

			// After applying impulse, constraint TOI need be updated to reflect the new velocities. 
			// Usually the new velocities are separating, and therefore TOI should be infinity.
			// See resweep below which (optionally) updates TOI for all other contacts as a result of handling this one
			CCDConstraint->SweptConstraint->ResetCCDTimeOfImpact();

			bool bMovedParticle0 = false;
			bool bMovedParticle1 = false;
			if (CCDConstraint->ProcessedCount >= CVars::ChaosCollisionCCDConstraintMaxProcessCount)
			{
				/* Here is how clipping works:
				* Assuming collision detection gives us all the possible collision pairs in the current frame.
				* Because we sort and apply constraints based on their TOIs, at current IslandTOI, the two particles cannot tunnel through other particles in the island. 
				* Now, we run out of the computational budget for this constraint, then we freeze the two particles in place. The current two particles cannot tunnel through each other this frame.
				* The two particles are then treated as static. When resweeping, we update TOIs of other constraints to make sure other particles in the island cannot tunnel through this two particles.
				* Therefore, by clipping, we can avoid tunneling but this is at the cost of reduced momentum.
				* For kinematic particles, we cannot freeze them in place. In this case, we simply offset the particle with the kinematic motion from [IslandTOI, 1] along the collision normal and freeze it there.
				* If collision detection is not perfect and does not give us all the secondary collision pairs, setting ChaosCollisionCCDConstraintMaxProcessCount to 1 will always prevent tunneling.
				*/ 
				if (CVars::bChaosCollisionCCDAllowClipping)
				{
					if (CCDParticle0)
					{
						if (CCDConstraint->FastMovingKinematicIndex != INDEX_NONE)
						{
							// @todo(chaos): can this just get the particle from the CCD constraint?
							const FConstGenericParticleHandle KinematicParticle = FGenericParticleHandle(CCDConstraint->SweptConstraint->GetParticle(CCDConstraint->FastMovingKinematicIndex));
							const FVec3 Normal = CCDConstraint->SweptConstraint->CalculateWorldContactNormal();
							const FVec3 Offset = FVec3::DotProduct(KinematicParticle->V() * ((1.f - IslandTOI) * Dt), Normal) * Normal;
							ClipParticleP(CCDParticle0, Offset);
						}
						else
						{
							ClipParticleP(CCDParticle0);
						}
						CCDParticle0->Done = true;
						bMovedParticle0 = true;
					}
					if (CCDParticle1)
					{
						if (CCDConstraint->FastMovingKinematicIndex != INDEX_NONE)
						{
							// @todo(chaos): can this just get the particle from the CCD constraint?
							const FConstGenericParticleHandle KinematicParticle = FGenericParticleHandle(CCDConstraint->SweptConstraint->GetParticle(CCDConstraint->FastMovingKinematicIndex));
							const FVec3 Normal = CCDConstraint->SweptConstraint->CalculateWorldContactNormal();
							const FVec3 Offset = FVec3::DotProduct(KinematicParticle->V() * ((1.f - IslandTOI) * Dt), Normal) * Normal;
							ClipParticleP(CCDParticle1, Offset);
						}
						else
						{
							ClipParticleP(CCDParticle1);
						}
						CCDParticle1->Done = true;
						bMovedParticle1 = true;
					}
				}
				// If clipping is not allowed, we update particle P (at TOI=1) based on new velocities. 
				else
				{
					if (CCDParticle0)
					{
						UpdateParticleP(CCDParticle0, Dt);
						bMovedParticle0 = true;
					}
					if (CCDParticle1)
					{
						UpdateParticleP(CCDParticle1, Dt);
						bMovedParticle1 = true;
					}
				}
				// Increment ConstraintIndex if we run out of computational budget for this constraint.
				ConstraintIndex ++;
			}
			// If we still have computational budget for this constraint, update particle P and don't clip.
			else
			{
				if (CCDParticle0 && !CCDParticle0->Done)
				{
					UpdateParticleP(CCDParticle0, Dt);
					bMovedParticle0 = true;
				}
				if (CCDParticle1 && !CCDParticle1->Done)
				{
					UpdateParticleP(CCDParticle1, Dt);
					bMovedParticle1 = true;
				}
			}

			// We applied a CCD impulse and updated the particle positions, so we need to update all the constraints involving these particles
			bool bHasResweptConstraint = false;
			if (bMovedParticle0)
			{
				bHasResweptConstraint |= UpdateParticleSweptConstraints(CCDParticle0, IslandTOI, Dt);
			}
			if (bMovedParticle1)
			{
				bHasResweptConstraint |= UpdateParticleSweptConstraints(CCDParticle1, IslandTOI, Dt);
			}

			// If we updated some constraints, we need to sort so that we handle the next TOI event
			if (bHasResweptConstraint)
			{
				std::sort(SortedCCDConstraints.GetData() + ConstraintIndex, SortedCCDConstraints.GetData() + ConstraintStart + ConstraintNum, CCDConstraintSortPredicate);
			}
		}
		
		// Update the constraint with the CCD results
		for (FCCDConstraint* CCDConstraint : SortedCCDConstraints)
		{ 
			CCDConstraint->SweptConstraint->SetCCDResults(CCDConstraint->NetImpulse);
		}
	}

	bool FCCDManager::UpdateParticleSweptConstraints(FCCDParticle* CCDParticle, const FReal IslandTOI, const FReal Dt)
	{
		const FReal RestDt = (1.f - IslandTOI) * Dt;
		bool HasResweptConstraint = false;
		if (CCDParticle != nullptr)
		{
			for (int32 AttachedCCDConstraintIndex = 0; AttachedCCDConstraintIndex < CCDParticle->AttachedCCDConstraints.Num(); AttachedCCDConstraintIndex++)
			{
				FCCDConstraint* AttachedCCDConstraint = CCDParticle->AttachedCCDConstraints[AttachedCCDConstraintIndex];
				if (AttachedCCDConstraint->ProcessedCount >= CVars::ChaosCollisionCCDConstraintMaxProcessCount)
				{
					continue;
				}

				const bool bParticle0Done = ((AttachedCCDConstraint->Particle[0] == nullptr) || AttachedCCDConstraint->Particle[0]->Done);
				const bool bParticle1Done = ((AttachedCCDConstraint->Particle[1] == nullptr) || AttachedCCDConstraint->Particle[1]->Done);
				if (bParticle0Done && bParticle1Done)
				{
					continue;
				}

				// Particle transforms at TOI
				FRigidTransform3 ParticleStartWorldTransforms[2];
				for (int32 j = 0; j < 2; j++)
				{
					FCCDParticle* AffectedCCDParticle = AttachedCCDConstraint->Particle[j];
					if (AffectedCCDParticle != nullptr)
					{
						TPBDRigidParticleHandle<FReal, 3>* AffectedParticle = AffectedCCDParticle->Particle;
						if (!AffectedCCDParticle->Done)
						{
							AdvanceParticleXToTOI(AffectedCCDParticle, IslandTOI, Dt);
						}
						ParticleStartWorldTransforms[j] = FRigidTransform3(AffectedParticle->GetX(), AffectedParticle->GetR());
					}
					else
					{
						FGenericParticleHandle AffectedParticle = FGenericParticleHandle(AttachedCCDConstraint->SweptConstraint->GetParticle(j));
						const bool IsKinematic = AffectedParticle->ObjectState() == EObjectStateType::Kinematic;
						if (IsKinematic)
						{
							ParticleStartWorldTransforms[j] = FRigidTransform3(AffectedParticle->P() - AffectedParticle->V() * RestDt, AffectedParticle->Q());
						}
						else // Static case
						{
							ParticleStartWorldTransforms[j] = FRigidTransform3(AffectedParticle->GetX(), AffectedParticle->GetR());
						}
					}
				}
				/** When resweeping, we need to recompute TOI for affected constraints and therefore the work (GJKRaycast) used to compute the original TOI is wasted.
				* A potential optimization is to compute an estimate of TOI using the AABB of the particles. Sweeping AABBs to compute an estimated TOI can be very efficient, and this TOI is strictly smaller than the accurate TOI.
				* At each for-loop iteration, we only need the constraint with the smallest TOI in the island. A potential optimized algorithm could be like:
				* 	First, sort constraints based on estimated TOI.
				*	Find the constraint with the smallest accurate TOI:
				*		Walk through the constraint list, change estimated TOI to accurate TOI
				*		If accurate TOI is smaller than estimated TOI of the next constraint, we know we found the constraint.
				*	When resweeping, compute estimated TOI instead of accurate TOI since updated TOI might need to be updated again.
				*/

				FPBDCollisionConstraint* SweptConstraint = AttachedCCDConstraint->SweptConstraint;
				const FConstGenericParticleHandle Particle0 = SweptConstraint->GetParticle0();
				const FConstGenericParticleHandle Particle1 = SweptConstraint->GetParticle1();

				// Initial shape sweep transforms
				const FRigidTransform3 ShapeStartWorldTransform0 = SweptConstraint->GetShapeRelativeTransform0() * ParticleStartWorldTransforms[0];
				const FRigidTransform3 ShapeStartWorldTransform1 = SweptConstraint->GetShapeRelativeTransform1() * ParticleStartWorldTransforms[1];

				// End shape sweep transforms
				const FRigidTransform3 ParticleEndWorldTransform0 = FParticleUtilities::GetActorWorldTransform(Particle0);
				const FRigidTransform3 ParticleEndWorldTransform1 = FParticleUtilities::GetActorWorldTransform(Particle1);
				const FRigidTransform3 ShapeEndWorldTransform0 = SweptConstraint->GetShapeRelativeTransform0() * ParticleEndWorldTransform0;
				const FRigidTransform3 ShapeEndWorldTransform1 = SweptConstraint->GetShapeRelativeTransform1() * ParticleEndWorldTransform1;

				// Update of swept constraint assumes that the constraint holds the end transforms for the sweep
				SweptConstraint->SetShapeWorldTransforms(ShapeEndWorldTransform0, ShapeEndWorldTransform1);

				// Calculate the contact point and TOI
				Collisions::UpdateConstraintSwept(*SweptConstraint, ShapeStartWorldTransform0, ShapeStartWorldTransform1, RestDt);

				const FReal RestDtTOI = AttachedCCDConstraint->SweptConstraint->GetCCDTimeOfImpact();
				if ((RestDtTOI >= 0) && (RestDtTOI < FReal(1)))
				{
					AttachedCCDConstraint->SweptConstraint->SetCCDTimeOfImpact(IslandTOI + (FReal(1) - IslandTOI) * RestDtTOI);
				}

				// When bUpdated==true, TOI was modified. When bUpdated==false, TOI was set to be TNumericLimits<FReal>::Max(). In either case, a re-sorting on the constraints is needed.
				HasResweptConstraint = true;
			}
		}
		
		for (FCCDConstraint* CCDConstraint : SortedCCDConstraints)
		{ 
			if (CCDConstraint && CCDConstraint->SweptConstraint)
			{
				CCDConstraint->SweptConstraint->SetCCDResults(CCDConstraint->NetImpulse);
			}
		}
		
		return HasResweptConstraint;
	}

	void FCCDManager::ResetIslandParticles(const int32 Island)
	{
		const int32 ParticleStart = IslandParticleStart[Island];
		const int32 ParticleNum = IslandParticleNum[Island];
		for (int32 i = ParticleStart; i < ParticleStart + ParticleNum; i++)
		{
			GroupedCCDParticles[i]->TOI = 0.f;
			GroupedCCDParticles[i]->Done = false;
		}
	}

	void FCCDManager::ResetIslandConstraints(const int32 Island)
	{
		const int32 ConstraintStart = IslandConstraintStart[Island];
		const int32 ConstraintEnd = IslandConstraintEnd[Island];
		for (int32 i = ConstraintStart; i < ConstraintEnd; i++)
		{
			SortedCCDConstraints[i]->ProcessedCount = 0;
		}
	}

	void FCCDManager::AdvanceParticleXToTOI(FCCDParticle *CCDParticle, const FReal TOI, const FReal Dt) const
	{
		if (TOI > CCDParticle->TOI)
		{
			TPBDRigidParticleHandle<FReal, 3>* Particle = CCDParticle->Particle;
			const FReal RestDt = (TOI - CCDParticle->TOI) * Dt;
			Particle->SetX(Particle->GetX() + Particle->GetV() * RestDt);
			CCDParticle->TOI = TOI;
		}
	}

	void FCCDManager::UpdateParticleP(FCCDParticle *CCDParticle, const FReal Dt) const
	{
		TPBDRigidParticleHandle<FReal, 3>* Particle = CCDParticle->Particle;
		const FReal RestDt = (1.f - CCDParticle->TOI) * Dt;
		Particle->SetP(Particle->GetX() + Particle->GetV() * RestDt);
	}

	void FCCDManager::ClipParticleP(FCCDParticle *CCDParticle) const
	{
		TPBDRigidParticleHandle<FReal, 3>* Particle = CCDParticle->Particle;
		Particle->SetP(Particle->GetX());
	}

	void FCCDManager::ClipParticleP(FCCDParticle *CCDParticle, const FVec3 Offset) const
	{
		TPBDRigidParticleHandle<FReal, 3>* Particle = CCDParticle->Particle;
		Particle->SetX(Particle->GetX() + Offset);
		Particle->SetP(Particle->GetX());
	}

	void FCCDManager::ApplyImpulse(FCCDConstraint *CCDConstraint)
	{
		FPBDCollisionConstraint *Constraint = CCDConstraint->SweptConstraint;
		TPBDRigidParticleHandle<FReal, 3> *Rigid0 = Constraint->GetParticle0()->CastToRigidParticle();
		TPBDRigidParticleHandle<FReal, 3> *Rigid1 = Constraint->GetParticle1()->CastToRigidParticle();
		check(Rigid0 != nullptr || Rigid1 != nullptr);
		const FReal Restitution = Constraint->GetRestitution();
		const FRigidTransform3& ShapeWorldTransform1 = Constraint->GetShapeWorldTransform1();
		for (int32 ManifoldPointIndex = 0; ManifoldPointIndex < Constraint->NumManifoldPoints(); ++ManifoldPointIndex)
		{
			const FManifoldPoint& ManifoldPoint = Constraint->GetManifoldPoint(ManifoldPointIndex);
			if (ManifoldPoint.Flags.bDisabled)
			{
				continue;
			}
			
			const FVec3 Normal = ShapeWorldTransform1.TransformVectorNoScale(FVec3(ManifoldPoint.ContactPoint.ShapeContactNormal));
			const FVec3 V0 = Rigid0 != nullptr ? Rigid0->GetV() : FVec3(0.f);
			const FVec3 V1 = Rigid1 != nullptr ? Rigid1->GetV() : FVec3(0.f);
			const FReal NormalV = FVec3::DotProduct(V0 - V1, Normal);
			if (NormalV < 0.f)
			{
				const FReal TargetNormalV = -Restitution * NormalV;
				// If a particle is marked done, we treat it as static by setting InvM to 0. 
				const bool bInfMass0 = Rigid0 == nullptr || (CVars::bChaosCollisionCCDAllowClipping && CCDConstraint->Particle[0] && CCDConstraint->Particle[0]->Done);
				const bool bInfMass1 = Rigid1 == nullptr || (CVars::bChaosCollisionCCDAllowClipping && CCDConstraint->Particle[1] && CCDConstraint->Particle[1]->Done);
				const FReal InvM0 = bInfMass0 ? 0.f : Rigid0->InvM();
				const FReal InvM1 = bInfMass1 ? 0.f : Rigid1->InvM();
				const FVec3 Impulse = (TargetNormalV - NormalV) * Normal / (InvM0 + InvM1);
				if (InvM0 > 0.f)
				{
					Rigid0->SetV(Rigid0->GetV() + Impulse * InvM0);
				}
				if (InvM1 > 0.f)
				{
					Rigid1->SetV(Rigid1->GetV() - Impulse * InvM1);
				}

				CCDConstraint->NetImpulse += Impulse;

#if CHAOS_DEBUG_DRAW
				if (CVars::ChaosSolverDrawCCDInteractions)
				{
					DebugDraw::DrawCCDCollisionImpulse(FRigidTransform3::Identity, *CCDConstraint, ManifoldPointIndex, Impulse, &CVars::ChaosSolverDebugDebugDrawSettings);
				}
#endif
			}
		}
	}

	void FCCDManager::UpdateSweptConstraints(const FReal Dt)
	{
		// Build the set of collision whose contact data will be out of date because we moved one or both of its particles. 
		// This is all collision constraints, including non-swept ones, for any particle that was relocated by the CCD sweep 
		// logic executed in ApplySweptConstraints (i.e., contents of CCDConstraints)
		// @todo(chaos): we could calculate the size of the Collisions array in Init
		// @todo(chaos): could be parallelized
		TSet<FPBDCollisionConstraint*> Collisions;
		for (FCCDParticle& CCDParticle : CCDParticles)
		{
			CCDParticle.Particle->ParticleCollisions().VisitCollisions(
				[&Collisions](FPBDCollisionConstraint& Collision)
				{
					Collisions.Add(&Collision);
					return ECollisionVisitorResult::Continue;
				});
		}

		// Update all the collisions
		// @todo(chaos): can be parallelized
		for (FPBDCollisionConstraint* Collision : Collisions)
		{
			const FConstGenericParticleHandle P0 = FConstGenericParticleHandle(Collision->GetParticle0());
			const FConstGenericParticleHandle P1 = FConstGenericParticleHandle(Collision->GetParticle1());

			const FRigidTransform3 ShapeWorldTransform0 = Collision->GetShapeRelativeTransform0() * P0->GetTransformPQ();
			const FRigidTransform3 ShapeWorldTransform1 = Collision->GetShapeRelativeTransform1() * P1->GetTransformPQ();
			Collision->SetShapeWorldTransforms(ShapeWorldTransform0, ShapeWorldTransform1);

			// NOTE: ResetManifold also reset friction anchors. If CCD sweep was run, static friction probably will not hold
			// We could potentially call ResetActiveManifold here instead and then AssignSavedManifoldPoints if we want static friction
			Collision->ResetManifold();
			Collisions::UpdateConstraint(*Collision, ShapeWorldTransform0, ShapeWorldTransform1, Dt);
		}
	}

	void FCCDManager::ProjectCCD(const FReal Dt, TArrayView<FPBDCollisionConstraint* const> SweptConstraints)
	{
		// NOTE: The SweptConstraints list may be different that the one used in ApplyCCD
		// earlier in the frame but that's ok - we still want to enforce depenetration.
		// - some may be removed (or nulled) if particles were destroyed
		// - some may be added if an island was awakened
		TArray<FPBDCollisionConstraint*> Constraints = TArray<FPBDCollisionConstraint*>(SweptConstraints);

		// Remove null constraints in the active list. Also check for null pointers (which is probably not required 
		// any more - we used to just null the particle pointers in collisions and let them auto-clean up later
		// but now we destroy them immediately).
		if(Chaos::CVars::bClearNullCCDConstraintsOnApply)
		{
			Constraints.RemoveAll(
				[](FPBDCollisionConstraint* C)
				{
					return (C == nullptr)
						|| (C->GetParticle0() == nullptr)
						|| (C->GetParticle1() == nullptr);
				}
			);
		}

		// Build a list of CCD constraints sorted so that the most important ones are last.
		// The primary goal is to prevent CCD objects from leaving the world, so we process collisions
		// with statics/kinematics last. The secondary goal is to prevent CCD objects passing through each 
		// other. And finally we try to prevent CCD object passing through non-CCD dynamics but aren't
		// too worried if that happens.
		//
		// Sort order based on the two particle types in the collision:
		//		CCD - Dynamic
		//		CCD - CCD
		//		CCD - Kinematic
		Constraints.Sort([](const FPBDCollisionConstraint& L, const FPBDCollisionConstraint& R)
			{
				const FGenericParticleHandle LP0 = L.GetParticle0();
				const FGenericParticleHandle LP1 = L.GetParticle1();
				const FGenericParticleHandle RP0 = R.GetParticle0();
				const FGenericParticleHandle RP1 = R.GetParticle1();

				const bool bLCCD0 = LP0->CCDEnabled();
				const bool bLCCD1 = LP1->CCDEnabled();
				const bool bRCCD0 = RP0->CCDEnabled();
				const bool bRCCD1 = RP1->CCDEnabled();
				const bool bLDynamic0 = LP0->IsDynamic();
				const bool bLDynamic1 = LP1->IsDynamic();
				const bool bRDynamic0 = RP0->IsDynamic();
				const bool bRDynamic1 = RP1->IsDynamic();

				// If only one constraint has a non-ccd-dynamic it goes before the other
				const bool bLHasNonCCDDynamic = (!bLCCD0 && bLDynamic0) || (!bLCCD1 && bLDynamic1);
				const bool bRHasNonCCDDynamic = (!bRCCD0 && bRDynamic0) || (!bRCCD1 && bRDynamic1);
				if (bLHasNonCCDDynamic != bRHasNonCCDDynamic)
				{
					return bLHasNonCCDDynamic;
				}

				// If only one constraint has a non-ccd-kinematic it goes after the other
				const bool bLHasNonCCDKinematic = (!bLCCD0 && !bLDynamic0) || (!bLCCD1 && !bLDynamic1);
				const bool bRHasNonCCDKinematic = (!bRCCD0 && !bRDynamic0) || (!bRCCD1 && !bRDynamic1);
				if (bLHasNonCCDKinematic != bRHasNonCCDKinematic)
				{
					return bRHasNonCCDKinematic;
				}

				// Otherwise we don't care about the order
				return false;
			});


		// Loop over all the CCD constraints and if the penetration depth is greater than some threshold,
		// move the two bodies so that they are no longer penetrating.
		// We iterate to handle stacks of CCD objects and extracting a CCD objects from a wedge.
		const FReal PhiToleranceScale = CVars::ChaosCollisionCCDCorrectionPhiToleranceScale;
		const int32 MaxIterations = CVars::ChaosCollisionCCDCorrectionIterations;
		int32 NumIterations = 0;
		bool bSolved = false;
		while (!bSolved && (NumIterations < MaxIterations))
		{
			bSolved = true;
			++NumIterations;

			for (FPBDCollisionConstraint* Constraint : Constraints)
			{
				if (Constraint->IsProbe() || !Constraint->IsEnabled())
				{
					continue;
				}

				const FGenericParticleHandle P0 = Constraint->GetParticle0();
				const FGenericParticleHandle P1 = Constraint->GetParticle1();
				const bool bDynamic0 = P0->IsDynamic();
				const bool bDynamic1 = P1->IsDynamic();
				const bool bCCD0 = P0->CCDEnabled();
				const bool bCCD1 = P1->CCDEnabled();

				// Skip no CCD (should not happen)
				if (!bCCD0 && !bCCD1)
				{
					continue;
				}

				// Skip two non-dynamics (should not happen)
				if (!bDynamic0 && !bDynamic1)
				{
					continue;
				}

				// For two dynamics we move each body 50% (we know we have at least one dynamic here)
				FReal Bias0 = FReal(0.5);
				if (!bDynamic0)
				{
					// Only body1 moves
					Bias0 = FReal(0);
				}
				else if (!bDynamic1)
				{
					// Only body0 moves
					Bias0 = FReal(1);
				}
				const FReal Bias1 = FReal(1) - Bias0;

				// This function is called after the solver phase - the shape transform and Phi are out of date
				const FRigidTransform3 ShapeWorldTransform0 = Constraint->GetShapeRelativeTransform0() * FRigidTransform3(P0->P(), P0->Q());
				const FRigidTransform3 ShapeWorldTransform1 = Constraint->GetShapeRelativeTransform1() * FRigidTransform3(P1->P(), P1->Q());
				Constraint->SetShapeWorldTransforms(ShapeWorldTransform0, ShapeWorldTransform1);
				Constraint->UpdateManifoldContacts();

				const FReal Phi = Constraint->GetPhi();
				const FReal PhiTolerance = PhiToleranceScale * Constraint->GetCCDTargetPenetration();

				FReal TargetSeparation = 0;
				if (const FManifoldPoint* ManifoldPoint = Constraint->GetClosestManifoldPoint())
				{
					TargetSeparation = ManifoldPoint->TargetPhi;
				}
				const FReal MinPhi = TargetSeparation - PhiTolerance;

				if (Phi < MinPhi)
				{
					bSolved = false;

					// Extract the two bodies along the normal and set the contact velocity to zero, 
					// as if we have two point particles and zero restitution.
					// NOTE: this is ignoring their relative masses and so does not conserve momentum
					const FVec3 WorldNormal = Constraint->CalculateWorldContactNormal();
					const FReal VelNormal = FMath::Min(FVec3::DotProduct(P0->V() - P1->V(), WorldNormal), FReal(0));

					if (Bias0 > UE_SMALL_NUMBER)
					{
						const FVec3 Correction0 = -Bias0 * Phi * WorldNormal;
						P0->SetP(P0->P() + Correction0);
						P0->SetV(P0->V() - Bias0 * VelNormal * WorldNormal);
					}

					if (Bias1 > UE_SMALL_NUMBER)
					{
						const FVec3 Correction1 = Bias1 * Phi * WorldNormal;
						P1->SetP(P1->P() + Correction1);
						P1->SetV(P1->V() + Bias1 * VelNormal * WorldNormal);
					}
				}
			}
		}
	}


	void FCCDManager::OverwriteXUsingV(const FReal Dt)
	{
		// Overwriting X = P - V * Dt so that the implicit velocity step will give our velocity back.
		for (FCCDParticle& CCDParticle : CCDParticles)
		{
			TPBDRigidParticleHandle<FReal, 3>* Particle = CCDParticle.Particle;
			Particle->SetX(Particle->GetP() - Particle->GetV() * Dt);
		}
	}


	//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////


	FRigidTransform3 FCCDHelpers::GetParticleTransformAtTOI(const FGeometryParticleHandle* InParticle, const FReal TOI, const FReal Dt)
	{
		// Particles are swept using their latest rotation. We interpolate using velocity rather that Lerp(X,P,TOI)
		// to handle the case where the center of mass is offset from the particle position.
		const FConstGenericParticleHandle Particle = InParticle;
		if (Particle->CCDEnabled())
		{
			const FReal InvClampedTOI = FMath::Clamp(FReal(1) - TOI, FReal(0), FReal(1));
			return FRigidTransform3(
				Particle->P() - (InvClampedTOI * Dt) * Particle->V(),
				Particle->Q());
		}
		return Particle->GetTransformXR();
	}

	bool FCCDHelpers::DeltaExceedsThreshold(const FVec3& AxisThreshold, const FVec3& DeltaX, const FQuat& R)
	{
		FVec3 AbsLocalDelta, AxisThresholdScaled, AxisThresholdDiff;
		return DeltaExceedsThreshold(AxisThreshold, DeltaX, R, AbsLocalDelta, AxisThresholdScaled, AxisThresholdDiff);
	}

	bool FCCDHelpers::DeltaExceedsThreshold(const FVec3& AxisThreshold, const FVec3& DeltaX, const FQuat& R, FVec3& OutAbsLocalDelta, FVec3& OutAxisThresholdScaled, FVec3& OutAxisThresholdDiff)
	{
		if (CVars::CCDEnableThresholdBoundsScale < 0.f) { return false; }
		if (CVars::CCDEnableThresholdBoundsScale == 0.f) { return true; }

		// Get per-component absolute value of position delta in local space.
		// This is how much we've moved on each principal axis (but not which
		// direction on that axis that we've moved in).
		OutAbsLocalDelta = R.UnrotateVector(DeltaX).GetAbs();

		// Scale the ccd extents in local space and subtract them from the 
		// local space position deltas. This will give us a vector representing
		// how much further we've moved on each axis than should be allowed by
		// the CCD bounds.
		OutAxisThresholdScaled = AxisThreshold * CVars::CCDEnableThresholdBoundsScale;
		OutAxisThresholdDiff = OutAbsLocalDelta - OutAxisThresholdScaled;

		// That is, if any element of ExtentsDiff is greater than zero, then that
		// means DeltaX has exceeded the scaled extents
		return OutAxisThresholdDiff.GetMax() > 0.f;
	}

	bool FCCDHelpers::DeltaExceedsThreshold(
		const FVec3& AxisThreshold0, const FVec3& DeltaX0, const FQuat& R0,
		const FVec3& AxisThreshold1, const FVec3& DeltaX1, const FQuat& R1)
	{
		return FCCDHelpers::DeltaExceedsThreshold(

			// To combine axis thresholds:
			// * transform particle1's threshold into particle0's local space
			// * take the per-component minimum of each axis threshold
			//
			// To think about why we use component mininma to combine thresholds,
			// imagine what happens when a large object and a small object move
			// towards each other at the same speed. Say particle0 is the large
			// object, and then think about particle1's motion from particle0's
			// inertial frame of reference. In this case, clearly you should
			// choose particle1's threshold since it is the one that is moving.
			//
			// Since there's no preferred inertial frame, the correct choice
			// will always be to take the smaller object's threshold.
			AxisThreshold0.ComponentMin((R0 * R1.UnrotateVector(AxisThreshold1)).GetAbs()),

			// Taking the difference of the deltas gives the total delta - how
			// much the objects have moved towards each other. We choose to use
			// particle0 as the reference.
			DeltaX1 - DeltaX0,

			// Since we're doing this in particle0's space, we choose its rotation.
			R0);
	}

	bool FCCDHelpers::DeltaExceedsThreshold(const FGeometryParticleHandle& Particle0, const FGeometryParticleHandle& Particle1)
	{
		// For rigids, compute DeltaX from the X - P diff and use Q for the rotation.
		// For non-rigids, DeltaX is zero and use R for rotation.
		const auto Rigid0 = Particle0.CastToRigidParticle();
		const auto Rigid1 = Particle1.CastToRigidParticle();
		const FVec3 DeltaX0 = Rigid0 ? Rigid0->GetP() - Rigid0->GetX() : FVec3::ZeroVector;
		const FVec3 DeltaX1 = Rigid1 ? Rigid1->GetP() - Rigid1->GetX() : FVec3::ZeroVector;
		const FQuat& R0 = Rigid0 ? Rigid0->GetQ() : Particle0.GetR();
		const FQuat& R1 = Rigid1 ? Rigid1->GetQ() : Particle1.GetR();
		return DeltaExceedsThreshold(
			Particle0.CCDAxisThreshold(), DeltaX0, R0,
			Particle1.CCDAxisThreshold(), DeltaX1, R1);
	}

	bool FCCDHelpers::DeltaExceedsThreshold(const FGeometryParticleHandle& Particle0, const FGeometryParticleHandle& Particle1, const FReal Dt)
	{
		// For rigids, compute DeltaX from the V * Dt and use Q for the rotation.
		// For non-rigids, DeltaX is zero and use R for rotation.
		const auto Rigid0 = Particle0.CastToRigidParticle();
		const auto Rigid1 = Particle1.CastToRigidParticle();
		const FVec3 DeltaX0 = Rigid0 ? Rigid0->GetV() * Dt : FVec3::ZeroVector;
		const FVec3 DeltaX1 = Rigid1 ? Rigid1->GetV() * Dt : FVec3::ZeroVector;
		const FQuat& R0 = Rigid0 ? Rigid0->GetQ() : Particle0.GetR();
		const FQuat& R1 = Rigid1 ? Rigid1->GetQ() : Particle1.GetR();
		return DeltaExceedsThreshold(
			Particle0.CCDAxisThreshold(), DeltaX0, R0,
			Particle1.CCDAxisThreshold(), DeltaX1, R1);
	}
}