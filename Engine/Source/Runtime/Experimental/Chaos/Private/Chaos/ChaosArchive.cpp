// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/ChaosArchive.h"
#include "Chaos/Serializable.h"
#include "Chaos/ImplicitObject.h"

namespace Chaos
{

FChaosArchiveContext::FChaosArchiveContext(): TagCount(0)
{
}

FChaosArchiveContext::~FChaosArchiveContext()
{
	for (auto Itr : ObjToSharedPtrHolder)
	{
		delete Itr.Value;
	}

	for (auto Itr : ObjToRefCountPtrHolder)
	{
		delete Itr.Value;
	}
}

void FChaosArchive::SerializeLegacy(TUniquePtr<FImplicitObject>& Obj)
{
	FImplicitObject::SerializeLegacyHelper(InnerArchive, Obj);
}

#if CHAOS_MEMORY_TRACKING
const FName FChaosArchiveMemoryTrackingContext::UntrackedName = TEXT("Untracked");

void FChaosArchiveMemoryTrackingContext::PushSection(const FName& SectionName, const int64 MemoryLocation, const bool bAbsorbChildren)
{
	check(SectionStack.Num() == 0 || SectionName != SectionStack.Top().Name);
	SectionStack.Push({ SectionName, MemoryLocation, 0, bAbsorbChildren });
	ChildAbsorbers += (int32)bAbsorbChildren;
}

void FChaosArchiveMemoryTrackingContext::PopSection(const int64 MemoryLocation)
{
	// Remove & get the top section
	const FChaosArchiveSection Section = SectionStack.Pop(EAllowShrinking::No);

	// Decrement the number of absorbers if the top section was absorbing subsections
	ChildAbsorbers -= (int32)Section.bAbsorbChildren;

	// Measure the size of this archive section
	const int64 SizeInclusive = MemoryLocation - Section.MemoryLocation;
	const int64 SizeExclusive = SizeInclusive - Section.ChildMemory;

	// Only count this section if it was at the bottom of the stack,
	// to avoid double counting.
	if (ChildAbsorbers == 0)
	{
		// Update this section's data
		FChaosArchiveSectionData& SectionData = SectionMap.FindOrAdd(Section.Name);
		SectionData.Count += 1;
		SectionData.SizeInclusive += SizeInclusive;
		SectionData.SizeExclusive += SizeExclusive;

		// Remove the size of this child from all parent sections
		for (FChaosArchiveSection& Parent : SectionStack)
		{
			Parent.ChildMemory += SizeExclusive;
		}
	}
}

void FChaosArchiveMemoryTrackingContext::BeginSerialize(const int64 MemoryLocation)
{
	if (SectionStack.Num() == 0)
	{
		PushSection(UntrackedName, MemoryLocation, false);
	}
}

void FChaosArchiveMemoryTrackingContext::EndSerialize(const int64 MemoryLocation)
{
	if (SectionStack.Top().Name == UntrackedName)
	{
		PopSection(MemoryLocation);
	}
}
#endif

}