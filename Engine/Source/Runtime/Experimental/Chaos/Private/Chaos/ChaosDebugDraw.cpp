// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/ChaosDebugDraw.h"
#include "Chaos/Box.h"
#include "Chaos/Capsule.h"
#include "Chaos/CCDUtilities.h"
#include "Chaos/Character/CharacterGroundConstraintContainer.h"
#include "Chaos/Collision/CollisionConstraintAllocator.h"
#include "Chaos/Collision/ParticlePairMidPhase.h"
#include "Chaos/Convex.h"
#include "Chaos/DebugDraw/DebugDrawPrimitives.h"
#include "Chaos/HeightField.h"
#include "Chaos/Triangle.h"
#include "Chaos/TriangleMesh.h"
#include "Chaos/DebugDrawQueue.h"
#include "Chaos/Evolution/SimulationSpace.h"
#include "Chaos/ImplicitObjectBVH.h"
#include "Chaos/ImplicitObjectScaled.h"
#include "Chaos/ImplicitObjectTransformed.h"
#include "Chaos/ImplicitObjectUnion.h"
#include "Chaos/Levelset.h"
#include "Chaos/Particle/ParticleUtilities.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/PBDCollisionConstraints.h"
#include "Chaos/PBDJointConstraints.h"
#include "Chaos/PBDRigidClustering.h"
#include "Chaos/PBDRigidParticles.h"
#include "Chaos/PBDSuspensionConstraints.h"
#include "Chaos/Sphere.h"
#include "Chaos/ConvexOptimizer.h"
#include "Chaos/Utilities.h"
#include "Chaos/CCDUtilities.h"

//PRAGMA_DISABLE_OPTIMIZATION

#if CHAOS_DEBUG_DRAW

namespace Chaos
{
	extern bool bChaosDebugDraw_UseNewQueue;

	namespace CVars
	{
		extern int32 ChaosOneWayInteractionPairCollisionMode;
		extern int32 ChaosSolverDrawShapesShowStatic;
		extern int32 ChaosSolverDrawShapesShowKinematic;
		extern int32 ChaosSolverDrawShapesShowDynamic;
	}

	namespace DebugDraw
	{
		bool bChaosDebugDebugDrawShapeBounds = false;
		FAutoConsoleVariableRef CVarChaosDebugDrawShapeBounds(TEXT("p.Chaos.DebugDraw.ShowShapeBounds"), bChaosDebugDebugDrawShapeBounds, TEXT("Whether to show the bounds of each shape in DrawShapes"));

		bool bChaosDebugDebugDrawCollisionParticles = false;
		FAutoConsoleVariableRef CVarChaosDebugDrawShapeParticles(TEXT("p.Chaos.DebugDraw.ShowCollisionParticles"), bChaosDebugDebugDrawCollisionParticles, TEXT("Whether to show the collision particles if present"));

		bool bChaosDebugDebugDrawInactiveContacts = true;
		FAutoConsoleVariableRef CVarChaosDebugDrawInactiveContacts(TEXT("p.Chaos.DebugDraw.ShowInactiveContacts"), bChaosDebugDebugDrawInactiveContacts, TEXT("Whether to show inactive contacts (ones that contributed no impulses or pushout)"));

		bool bChaosDebugDebugDrawContactIterations = false;
		FAutoConsoleVariableRef CVarChaosDebugDrawContactIterations(TEXT("p.Chaos.DebugDraw.ShowContactIterations"), bChaosDebugDebugDrawContactIterations, TEXT("Whether to show an indicator of how many iterations a contact was active for"));

		bool bChaosDebugDebugDrawColorShapesByShapeType = false;
		FAutoConsoleVariableRef CVarChaosDebugDebugDrawColorShapesByShapeType(TEXT("p.Chaos.DebugDraw.ColorShapesByShapeType"), bChaosDebugDebugDrawColorShapesByShapeType, TEXT("Whether to use shape type to define the color of the shapes instead of using the particle state "));

		bool bChaosDebugDebugDrawColorShapesByIsland = false;
		FAutoConsoleVariableRef CVarChaosDebugDebugDrawColorShapesByIsland(TEXT("p.Chaos.DebugDraw.ColorShapesByIsland"), bChaosDebugDebugDrawColorShapesByIsland, TEXT("Whether to use particle island to define the color of the shapes instead of using the particle state "));

		bool bChaosDebugDebugDrawColorShapesByInternalCluster = false;
		FAutoConsoleVariableRef CVarChaosDebugDebugDrawColorShapesByInternalCluster(TEXT("p.Chaos.DebugDraw.ColorShapesByInternalCluster"), bChaosDebugDebugDrawColorShapesByInternalCluster, TEXT("Whether to check if the particle is an internal cluster to define its color (black : regular particle: red :internal cluster )"));

		bool bChaosDebugDebugDrawColorShapesBySimQueryType = false;
		FAutoConsoleVariableRef CVarChaosDebugDebugDrawColorShapesBySimQueryType(TEXT("p.Chaos.DebugDraw.ColorShapesBySimQueryType"), bChaosDebugDebugDrawColorShapesBySimQueryType, TEXT("Whether to show with different colors shapes that are sim enabled and query enabled (sim : blue, query : orange)"));

		bool bChaosDebugDebugDrawColorShapesByConvexType = false;
		FAutoConsoleVariableRef CVarChaosDebugDebugDrawColorShapesByConvexType(TEXT("p.Chaos.DebugDraw.ColorShapesByConvexType"), bChaosDebugDebugDrawColorShapesByConvexType, TEXT("Whether to show with different colors shapes that are convex and simplified (simplified : green, normal : orange)"));

		bool bChaosDebugDebugDrawColorShapesByClusterUnion = false;
		FAutoConsoleVariableRef CVarChaosDebugDebugDrawColorShapesByClusterUnion(TEXT("p.Chaos.DebugDraw.ColorShapesByClusterUnion"), bChaosDebugDebugDrawColorShapesByClusterUnion, TEXT("An extension of the ColorShapesByInternalCluster option: instead of using a single color for every internal cluster, will use a unique color per cluster union. Non-cluster unions will be black."));

		bool bChaosDebugDebugDrawColorBoundsByShapeType = false;
		FAutoConsoleVariableRef CVarChaosDebugDebugDrawColorBoundsByShapeType(TEXT("p.Chaos.DebugDraw.ColorBoundsByShapeType"), bChaosDebugDebugDrawColorBoundsByShapeType, TEXT("Whether to use shape type to define the color of the bounds instead of using the particle state (if multiple shapes , will use the first one)"));

		bool bChaosDebugDebugDrawShowQueryOnlyShapes = true;
		FAutoConsoleVariableRef CVarChaosDebugDebugDrawShowQueryOnlyShapes(TEXT("p.Chaos.DebugDraw.ShowQueryOnly"), bChaosDebugDebugDrawShowQueryOnlyShapes, TEXT("Whether to show QueryOnly shapes"));

		bool bChaosDebugDebugDrawShowSimOnlyShapes = true;
		FAutoConsoleVariableRef CVarChaosDebugDebugDrawShowSimOnlyShapes(TEXT("p.Chaos.DebugDraw.ShowSimOnly"), bChaosDebugDebugDrawShowSimOnlyShapes, TEXT("Whether to show SimOnly shapes"));

		bool bChaosDebugDebugDrawShowProbeOnlyShapes = true;
		FAutoConsoleVariableRef CVarChaosDebugDebugDrawShowProbeOnlyShapes(TEXT("p.Chaos.DebugDraw.ShowProbeOnly"), bChaosDebugDebugDrawShowProbeOnlyShapes, TEXT("Whether to show ProbeOnly shapes"));

		bool bChaosDebugDebugDrawConvexVertices = false;
		bool bChaosDebugDebugDrawCoreShapes = false;
		bool bChaosDebugDebugDrawExactCoreShapes = false;
		FAutoConsoleVariableRef CVarChaosDebugDrawConvexVertices(TEXT("p.Chaos.DebugDraw.ShowConvexVertices"), bChaosDebugDebugDrawConvexVertices, TEXT("Whether to show the vertices of convex shapes"));
		FAutoConsoleVariableRef CVarChaosDebugDrawCoreShapes(TEXT("p.Chaos.DebugDraw.ShowCoreShapes"), bChaosDebugDebugDrawCoreShapes, TEXT("Whether to show the core (margin-reduced) shape where applicable"));
		FAutoConsoleVariableRef CVarChaosDebugDrawExactShapes(TEXT("p.Chaos.DebugDraw.ShowExactCoreShapes"), bChaosDebugDebugDrawExactCoreShapes, TEXT("Whether to show the exact core shape. NOTE: Extremely expensive and should only be used on a small scene with a couple convex shapes in it"));

		bool bChaosDebugDebugDrawShowOptimizedConvexes = false;
		FAutoConsoleVariableRef CVarChaosDebugDrawShowOptimizedConvexes(TEXT("p.Chaos.DebugDraw.ShowOptimizedConvexes"), bChaosDebugDebugDrawShowOptimizedConvexes, TEXT(""));

		bool bChaosDebugDebugDrawIslands = true;
		FAutoConsoleVariableRef CVarChaosDebugDrawIslands(TEXT("p.Chaos.DebugDraw.ShowIslands"), bChaosDebugDebugDrawIslands, TEXT("Whether to show the iosland boxes when drawing islands (if you want only the contact graph)"));

		bool bChaosDebugDebugDrawContactGraph = false;
		FAutoConsoleVariableRef CVarChaosDebugDrawContactGraph(TEXT("p.Chaos.DebugDraw.ShowContactGraph"), bChaosDebugDebugDrawContactGraph, TEXT("Whether to show the contactgraph when drawing islands"));

		bool bChaosDebugDebugDrawContactGraphUsed = false;
		FAutoConsoleVariableRef CVarChaosDebugDrawContactGraphUsed(TEXT("p.Chaos.DebugDraw.ShowContactGraphUsed"), bChaosDebugDebugDrawContactGraphUsed, TEXT("Whether to show the used edges contactgraph when drawing islands (collisions with impulse)"));

		bool bChaosDebugDebugDrawContactGraphUnused = false;
		FAutoConsoleVariableRef CVarChaosDebugDrawContactGraphUnused(TEXT("p.Chaos.DebugDraw.ShowContactGraphUnused"), bChaosDebugDebugDrawContactGraphUnused, TEXT("Whether to show the unused edges contactgraph when drawing islands (collisions with no impulse)"));

		float ChaosDebugDrawConvexExplodeDistance = 0.0f;
		FAutoConsoleVariableRef CVarChaosDebugDrawConvexExplodeDistance(TEXT("p.Chaos.DebugDraw.ConvexExplodeDistance"), ChaosDebugDrawConvexExplodeDistance, TEXT("Explode convex edges by this amount (useful for looking at convex integrity)"));

		float ChaosDebugDrawCCDDuration = 0.0f;
		FAutoConsoleVariableRef CVarChaosDebugDrawCCDDuration(TEXT("p.Chaos.DebugDraw.CCDDuration"), ChaosDebugDrawCCDDuration, TEXT("How long CCD debug draw should remain on screen in seconds. 0 for 1 frame."));

		float ChaosDebugDrawCollisionDuration = 0.0f;
		FAutoConsoleVariableRef CVarChaosDebugDrawCollisionDuration(TEXT("p.Chaos.DebugDraw.CollisionDuration"), ChaosDebugDrawCollisionDuration, TEXT("How long Collision debug draw should remain on screen in seconds. 0 for 1 frame."));

		bool bChaosDebugDrawCollisionAnchors = 0.0f;
		FAutoConsoleVariableRef CVarChaosDebugDrawCollisionAnchors(TEXT("p.Chaos.DebugDraw.ShowCollisionAnchors"), bChaosDebugDrawCollisionAnchors, TEXT(""));

		int32 ChaosConnectionGraphDrawLevelOffset = 0;
		FAutoConsoleVariableRef CVarChaosConnectionGraphDrawLevelOffset(TEXT("p.Chaos.DebugDraw.ConnectionGraphLevelOffset"), ChaosConnectionGraphDrawLevelOffset, TEXT("If 0, draws the connection graph between children particles of active particles. If 1, draws the connection grpah between grand-children particles of active particles, etc."));

		bool bChaosDebugDrawConnectionGraphShowAreas = false;
		FAutoConsoleVariableRef CVarChaosDebugDrawConnectionGraphShowAreas(TEXT("p.Chaos.DebugDraw.ConnectionGraph.ShowAreas"), bChaosDebugDrawConnectionGraphShowAreas, TEXT("When displaying connection graphs show connection areas as disks"));

		bool bChaosDebugDrawConnectionGraphShowInternalStrains = false;
		FAutoConsoleVariableRef CVarChaosDebugDrawConnectionGraphShowInternalStrains(TEXT("p.Chaos.DebugDraw.ConnectionGraph.ShowInternalStrains"), bChaosDebugDrawConnectionGraphShowInternalStrains, TEXT("When displaying connection graphs show strain values of each node"));

		int32 ChaosDebugDebugDrawBVHLevel = -1;
		bool bChaosDebugDebugDrawBVHShapes = false;
		bool bChaosDebugDebugDrawBVHBounds = true;
		FAutoConsoleVariableRef CVarChaosDebugDebugDrawBVHLevel(TEXT("p.Chaos.DebugDraw.BVHLevel"), ChaosDebugDebugDrawBVHLevel, TEXT("Which level of the BVH to draw (-1 = Leaf)"));
		FAutoConsoleVariableRef CVarChaosDebugDebugDrawBVHShapes(TEXT("p.Chaos.DebugDraw.BVHShapes"), bChaosDebugDebugDrawBVHShapes, TEXT("Whether to draw the shapes in each BVH node (colorized)"));
		FAutoConsoleVariableRef CVarChaosDebugDebugDrawBVHBounds(TEXT("p.Chaos.DebugDraw.BVHBounds"), bChaosDebugDebugDrawBVHBounds, TEXT("Whether to draw the BVH node bounds (colorized)"));

		//
		// TEMP - helpers to switch between new and old debug draw systems until we drop the old one
		// NOTE: This is switching to FChaosDDPrimitives, which misses out on the benefits of the new
		// system which are reduced data size and render-time style, but it will do for a start.
		// The end game is to replace use of FChaosDDPrimitives with specialized debugh draw
		// commands like FChaosDDParticles::DrawShapes() etc.
		//

		void DrawDebugPoint(const FVector& Position, const FColor& Color, bool bPersistentLines, float LifeTime, uint8 DepthPriority, float Thickness)
		{
			if (bChaosDebugDraw_UseNewQueue)
			{
				FChaosDDPrimitives::DrawPoint(Position, Color, Thickness, LifeTime);
			}
			else
			{
				FDebugDrawQueue::GetInstance().DrawDebugPoint(Position, Color, bPersistentLines, LifeTime, DepthPriority, Thickness);
			}
		}

		void DrawDebugLine(const FVector& LineStart, const FVector& LineEnd, const FColor& Color, bool bPersistentLines, float LifeTime, uint8 DepthPriority, float Thickness)
		{
			if (bChaosDebugDraw_UseNewQueue)
			{
				FChaosDDPrimitives::DrawLine(LineStart, LineEnd, Color, Thickness, LifeTime);
			}
			else
			{
				FDebugDrawQueue::GetInstance().DrawDebugLine(LineStart, LineEnd, Color, bPersistentLines, LifeTime, DepthPriority, Thickness);
			}
		}

		void DrawDebugSphere(FVector const& Center, FReal Radius, int32 Segments, const FColor& Color, bool bPersistentLines, float LifeTime, uint8 DepthPriority, float Thickness)
		{
			if (bChaosDebugDraw_UseNewQueue)
			{
				FChaosDDPrimitives::DrawSphere(Center, float(Radius), Color, Thickness, LifeTime);
			}
			else
			{
				FDebugDrawQueue::GetInstance().DrawDebugSphere(Center, Radius, Segments, Color, bPersistentLines, LifeTime, DepthPriority, Thickness);
			}
		}

		void DrawDebugBox(FVector const& Center, FVector const& Extent, const FQuat& Rotation, FColor const& Color, bool bPersistentLines, float LifeTime, uint8 DepthPriority, float Thickness)
		{
			if (bChaosDebugDraw_UseNewQueue)
			{
				FChaosDDPrimitives::DrawBox(Center, Rotation, Extent, Color, Thickness, LifeTime);
			}
			else
			{
				FDebugDrawQueue::GetInstance().DrawDebugBox(Center, Extent, Rotation, Color, bPersistentLines, LifeTime, DepthPriority, Thickness);
			}
		}

		void DrawDebugString(FVector const& TextLocation, const FString& Text, class AActor* TestBaseActor, FColor const& Color, float LifeTime, bool bDrawShadow, float FontScale)
		{
			if (bChaosDebugDraw_UseNewQueue)
			{
				FChaosDDPrimitives::DrawString(TextLocation, Text, Color, FontScale, bDrawShadow, LifeTime);
			}
			else
			{
				FDebugDrawQueue::GetInstance().DrawDebugString(TextLocation, Text, TestBaseActor, Color, LifeTime, bDrawShadow, FontScale);
			}
		}

		void DrawDebugCircle(const FVector& Center, FReal Radius, int32 Segments, const FColor& Color, bool bPersistentLines, float LifeTime, uint8 DepthPriority, float Thickness, const FMatrix& Axes, bool bDrawAxis)
		{
			if (bChaosDebugDraw_UseNewQueue)
			{
				FChaosDDPrimitives::DrawCircle(Center, Axes, float(Radius), Color, Thickness, LifeTime);
			}
			else
			{
				FDebugDrawQueue::GetInstance().DrawDebugCircle(Center, Radius, Segments, Color, bPersistentLines, LifeTime, DepthPriority, Thickness, Axes.GetUnitAxis(EAxis::Y), Axes.GetUnitAxis(EAxis::Z), bDrawAxis);
			}
		}

		void DrawDebugCapsule(const FVector& Center, FReal HalfHeight, FReal Radius, const FQuat& Rotation, const FColor& Color, bool bPersistentLines, float LifeTime, uint8 DepthPriority, float Thickness)
		{
			if (bChaosDebugDraw_UseNewQueue)
			{
				FChaosDDPrimitives::DrawCapsule(Center, Rotation, float(HalfHeight), float(Radius), Color, Thickness, LifeTime);
			}
			else
			{
				FDebugDrawQueue::GetInstance().DrawDebugCapsule(Center, HalfHeight, Radius, Rotation, Color, bPersistentLines, LifeTime, DepthPriority, Thickness);
			}
		}

		void DrawDebugDirectionalArrow(const FVector& LineStart, const FVector& LineEnd, float ArrowSize, const FColor& Color, bool bPersistentLines, float LifeTime, uint8 DepthPriority, float Thickness)
		{
			if (bChaosDebugDraw_UseNewQueue)
			{
				FChaosDDPrimitives::DrawArrow(LineStart, LineEnd, ArrowSize, Color, Thickness, LifeTime);
			}
			else
			{
				FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow(LineStart, LineEnd, ArrowSize, Color, bPersistentLines, LifeTime, DepthPriority, Thickness);
			}
		}

		FVector GetCenterOfInterest()
		{
			if (bChaosDebugDraw_UseNewQueue)
			{
				return ChaosDD::Private::FChaosDDContext::GetWriter().GetDrawRegion().Center;
			}
			else
			{
				return FDebugDrawQueue::GetInstance().GetCenterOfInterest();
			}
		}

		FReal GetRadiusOfInterest()
		{
			if (bChaosDebugDraw_UseNewQueue)
			{
				return ChaosDD::Private::FChaosDDContext::GetWriter().GetDrawRegion().W;
			}
			else
			{
				return FDebugDrawQueue::GetInstance().GetRadiusOfInterest();
			}
		}

		bool IsInRegionOfInterest(FVector Pos)
		{
			if (bChaosDebugDraw_UseNewQueue)
			{
				return ChaosDD::Private::FChaosDDContext::GetWriter().IsInDrawRegion(Pos);
			}
			else
			{
				return FDebugDrawQueue::GetInstance().IsInRegionOfInterest(Pos);
			}
		}

		bool IsInRegionOfInterest(FVector Pos, FReal Radius)
		{
			if (bChaosDebugDraw_UseNewQueue)
			{
				return ChaosDD::Private::FChaosDDContext::GetWriter().IsInDrawRegion(FSphere3d(Pos, Radius));
			}
			else
			{
				return FDebugDrawQueue::GetInstance().IsInRegionOfInterest(Pos, Radius);
			}
		}

		bool IsInRegionOfInterest(FAABB3 Bounds)
		{
			if (bChaosDebugDraw_UseNewQueue)
			{
				return ChaosDD::Private::FChaosDDContext::GetWriter().IsInDrawRegion(FBox3d(Bounds.Min(), Bounds.Max()));
			}
			else
			{
				return FDebugDrawQueue::GetInstance().IsInRegionOfInterest(Bounds);
			}
		}


		// NOTE: These settings should never really be used - they are the fallback defaults
		// if the user does not specify settings in the debug draw call.
		// See PBDRigidsColver.cpp and ImmediatePhysicsSimulation_Chaos.cpp for example.
		FChaosDebugDrawSettings ChaosDefaultDebugDebugDrawSettings(
			/* ArrowSize =					*/ 1.5f,
			/* BodyAxisLen =				*/ 4.0f,
			/* ContactLen =					*/ 4.0f,
			/* ContactWidth =				*/ 2.0f,
			/* ContactInfoWidth				*/ 2.0f,
			/* ContactOwnerWidth =			*/ 0.0f,
			/* ConstraintAxisLen =			*/ 5.0f,
			/* JointComSize =				*/ 2.0f,
			/* LineThickness =				*/ 0.15f,
			/* DrawScale =					*/ 1.0f,
			/* FontHeight =					*/ 10.0f,
			/* FontScale =					*/ 1.5f,
			/* ShapeThicknesScale =			*/ 1.0f,
			/* PointSize =					*/ 2.0f,
			/* VelScale =					*/ 0.0f,
			/* AngVelScale =				*/ 0.0f,
			/* ImpulseScale =				*/ 0.0f,
			/* PushOutScale =				*/ 0.0f,
			/* InertiaScale =				*/ 0.0f,
			/* DrawPriority =				*/ 10,
			/* bShowSimple =				*/ true,
			/* bShowComplex =				*/ false,
			/* bInShowLevelSetCollision =	*/ false,
			/* InShapesColorsPerState =     */ GetDefaultShapesColorsByState(),
			/* InShapesColorsPerShaepType=  */ GetDefaultShapesColorsByShapeType(),
			/* InBoundsColorsPerState =     */ GetDefaultBoundsColorsByState(),
			/* InBoundsColorsPerShapeType=  */ GetDefaultBoundsColorsByShapeType()
		);

		const FChaosDebugDrawSettings& GetChaosDebugDrawSettings(const FChaosDebugDrawSettings* InSettings)
		{
			if (InSettings != nullptr)
			{
				return *InSettings;
			}

			return ChaosDefaultDebugDebugDrawSettings;
		}

		//-------------------------------------------------------------------------------------------------

		FChaosDebugDrawColorsByState::FChaosDebugDrawColorsByState(
			FColor InDynamicColor,
			FColor InSleepingColor,
			FColor InKinematicColor,
			FColor InStaticColor,
			FColor InDebrisColor
		)
			: DynamicColor(InDynamicColor)
			, SleepingColor(InSleepingColor)
			, KinematicColor(InKinematicColor)
			, StaticColor(InStaticColor)
			, DebrisColor(InDebrisColor)
		{}

		FColor FChaosDebugDrawColorsByState::GetColorFromState(EObjectStateType State, bool bIsDebris) const
		{
			if (bIsDebris && (State != EObjectStateType::Sleeping))
			{
				return DebrisColor;
			}

			switch (State)
			{
			case EObjectStateType::Sleeping:	return SleepingColor;
			case EObjectStateType::Kinematic:	return KinematicColor;
			case EObjectStateType::Static:		return StaticColor;
			case EObjectStateType::Dynamic:		return DynamicColor;
			default:							return FColor::Purple; // nice visible color :)
			}
		}

		const FChaosDebugDrawColorsByState& GetDefaultShapesColorsByState()
		{
			// default colors by state for shapes
			static FChaosDebugDrawColorsByState ChaosDefaultShapesColorsByState
			{
				/* InDynamicColor =	  */ FColor(255, 255, 0),
				/* InSleepingColor =  */ FColor(128, 128, 128),
				/* InKinematicColor = */ FColor(0, 128, 255),
				/* InStaticColor =	  */ FColor(255, 0, 0),
				/* InDebrisColor =	  */ FColor(0, 255, 0),
			};

			return ChaosDefaultShapesColorsByState;
		}

		const FChaosDebugDrawColorsByState& GetDefaultBoundsColorsByState()
		{
			// default colors by state for bounds ( darker version of the shapes colors - see above )
			static FChaosDebugDrawColorsByState ChaosDefaultBoundsColorsByState
			{
				/* InDynamicColor =	  */ FColor(128, 128, 0),
				/* InSleepingColor =  */ FColor(64, 64, 64),
				/* InKinematicColor = */ FColor(0, 64, 128),
				/* InStaticColor =	  */ FColor(128, 0, 0),
				/* InDebrisColor =	  */ FColor(0, 128, 0),
			};

			return ChaosDefaultBoundsColorsByState;
		}

		CHAOS_API const FChaosDebugDrawColorsByState& GetDefaultShapesColorsPreIntegrate()
		{
			// default colors by state for shapes
			static FChaosDebugDrawColorsByState ChaosDefaultShapesColors
			{
				/* InDynamicColor =	  */ FColor(64, 64, 0),
				/* InSleepingColor =  */ FColor(0, 0, 0),
				/* InKinematicColor = */ FColor(0, 32, 64),
				/* InStaticColor =	  */ FColor(64, 0, 0),
				/* InDebrisColor =	  */ FColor(0, 64, 0),
			};

			return ChaosDefaultShapesColors;
		}

		const FChaosDebugDrawColorsByState& GetDefaultShapesColorsPostIntegrate()
		{
			// default colors by state for shapes
			static FChaosDebugDrawColorsByState ChaosDefaultShapesColors
			{
				/* InDynamicColor =	  */ FColor(255, 255, 0),
				/* InSleepingColor =  */ FColor(128, 128, 128),
				/* InKinematicColor = */ FColor(0, 128, 255),
				/* InStaticColor =	  */ FColor(255, 0, 0),
				/* InDebrisColor =	  */ FColor(0, 255, 0),
			};

			return ChaosDefaultShapesColors;
		}

		const FChaosDebugDrawColorsByState& GetDefaultShapesColorsCollisionDetection()
		{
			// default colors by state for shapes
			static FChaosDebugDrawColorsByState ChaosDefaultShapesColors
			{
				/* InDynamicColor =	  */ FColor(0, 255, 0),
				/* InSleepingColor =  */ FColor(0, 255, 0),
				/* InKinematicColor = */ FColor(0, 0, 0),
				/* InStaticColor =	  */ FColor(0, 0, 0),
				/* InDebrisColor =	  */ FColor(0, 255, 0),
			};

			return ChaosDefaultShapesColors;
		}

		FColor GetIndexColor(const int32 Index)
		{
			static const FColor Colors[] =
			{
				FColor::Red,
				FColor::Orange,
				FColor::Yellow,
				FColor::Green,
				FColor::Emerald,
				FColor::Cyan,
				FColor::Turquoise,
				FColor::Blue,
				FColor::Magenta,
				FColor::Purple,
			};
			const int32 NumColors = UE_ARRAY_COUNT(Colors);

			return Colors[Index % NumColors];
		};


		FColor GetIslandColor(const int32 IslandIndex, const bool bIsAwake)
		{
			static const FColor SleepingColor = FColor::Black;
			static const FColor NullColor = FColor::White;

			if (IslandIndex == INDEX_NONE)
			{
				return NullColor;
			}

			if (!bIsAwake)
			{
				return SleepingColor;
			}

			return GetIndexColor(IslandIndex);
		};


		//-------------------------------------------------------------------------------------------------

		FChaosDebugDrawColorsByShapeType::FChaosDebugDrawColorsByShapeType(
			FColor InSimpleTypeColor,
			FColor InConvexColor,
			FColor InHeightFieldColor,
			FColor InTriangleMeshColor,
			FColor InLevelSetColor
		)
			: SimpleTypeColor(InSimpleTypeColor)
			, ConvexColor(InConvexColor)
			, HeightFieldColor(InHeightFieldColor)
			, TriangleMeshColor(InTriangleMeshColor)
			, LevelSetColor(InLevelSetColor)
		{}

		FColor FChaosDebugDrawColorsByShapeType::GetColorFromShapeType(EImplicitObjectType ShapeType) const
		{
			switch(ShapeType)
			{
			case ImplicitObjectType::Sphere:			return SimpleTypeColor;
			case ImplicitObjectType::Box:				return SimpleTypeColor;
			case ImplicitObjectType::Plane:				return SimpleTypeColor;
			case ImplicitObjectType::Capsule:			return SimpleTypeColor;
			case ImplicitObjectType::TaperedCylinder:	return SimpleTypeColor;
			case ImplicitObjectType::Cylinder:			return SimpleTypeColor;
			case ImplicitObjectType::Convex:			return ConvexColor;
			case ImplicitObjectType::HeightField:		return HeightFieldColor;
			case ImplicitObjectType::TriangleMesh:		return TriangleMeshColor;
			case ImplicitObjectType::LevelSet:			return LevelSetColor;			
			default:									return FColor::Purple; // nice visible color :)
			};
		}

		CHAOS_API const FChaosDebugDrawColorsByShapeType& GetDefaultShapesColorsByShapeType()
		{
			// default colors by shaep type for shapes
			static FChaosDebugDrawColorsByShapeType ChaosDefaultShapesColorsByShapeType(
				/* InSimpleTypeColor,   */ FColor(0, 255, 0),
				/* InConvexColor,		*/ FColor(0, 255, 255),
				/* InHeightFieldColor,	*/ FColor(0, 0, 255),
				/* InTriangleMeshColor,	*/ FColor(255, 0, 0),
				/* InLevelSetColor		*/ FColor(255, 0, 128)
			);

			return ChaosDefaultShapesColorsByShapeType;
		}

		CHAOS_API const FChaosDebugDrawColorsByShapeType& GetDefaultBoundsColorsByShapeType()
		{
			// default colors by shape type for bounds ( darker version of the shapes colors - see above )
			static FChaosDebugDrawColorsByShapeType ChaosDefaultBoundsColorsByShapeType(
				/* InSimpleTypeColor,   */ FColor(0, 128, 0),
				/* InConvexColor,		*/ FColor(0, 128, 128),
				/* InHeightFieldColor,	*/ FColor(0, 0, 128),
				/* InTriangleMeshColor,	*/ FColor(128, 0, 0),
				/* InLevelSetColor		*/ FColor(128, 0, 64)
			);

			return ChaosDefaultBoundsColorsByShapeType;
		}

		//
		//
		//

		inline bool ParticleIsOneWayInteraction(const FGeometryParticleHandle* Particle)
		{
			if (const FPBDRigidParticleHandle* Rigid = Particle->CastToRigidParticle())
			{
				return Rigid->OneWayInteraction();
			}
			return false;
		}

		inline FColor GetParticleShapeDefaultColor(const FGeometryParticleHandle* Particle, const FChaosDebugDrawSettings& DebugDrawSettings)
		{
			return DebugDrawSettings.ShapesColorsPerState.GetColorFromState(Particle->ObjectState(), ParticleIsOneWayInteraction(Particle));
		}

		//
		//
		//

		bool IsParticleQueryOnly(const FGeometryParticleHandle* InParticle)
		{
			for (const FShapeInstancePtr& ShapeInstance : InParticle->ShapeInstances())
			{
				if (ShapeInstance->GetSimEnabled() || ShapeInstance->GetIsProbe())
				{
					return false;
				}
			}
			return true;
		}

		bool IsParticleSimOnly(const FGeometryParticleHandle* InParticle)
		{
			for (const FShapeInstancePtr& ShapeInstance : InParticle->ShapeInstances())
			{
				if (ShapeInstance->GetQueryEnabled() || ShapeInstance->GetIsProbe())
				{
					return false;
				}
			}
			return true;
		}

		bool IsParticleProbeOnly(const FGeometryParticleHandle* InParticle)
		{
			for (const FShapeInstancePtr& ShapeInstance : InParticle->ShapeInstances())
			{
				if (ShapeInstance->GetQueryEnabled() || ShapeInstance->GetSimEnabled())
				{
					return false;
				}
			}
			return true;
		}

		bool ShouldIgnoreParticle(const FGeometryParticleHandle* InParticle)
		{
			if (!CVars::ChaosSolverDrawShapesShowStatic)
			{
				if (FConstGenericParticleHandle(InParticle)->IsStatic())
				{
					return true;
				}
			}
			if (!CVars::ChaosSolverDrawShapesShowKinematic)
			{
				if (FConstGenericParticleHandle(InParticle)->IsKinematic())
				{
					return true;
				}
			}
			if (!CVars::ChaosSolverDrawShapesShowDynamic)
			{
				if (FConstGenericParticleHandle(InParticle)->IsDynamic())
				{
					return true;
				}
			}

			if (!bChaosDebugDebugDrawShowQueryOnlyShapes)
			{
				if (IsParticleQueryOnly(InParticle))
				{
					return true;
				}
			}
			if (!bChaosDebugDebugDrawShowSimOnlyShapes)
			{
				if (IsParticleSimOnly(InParticle))
				{
					return true;
				}
			}
			if (!bChaosDebugDebugDrawShowProbeOnlyShapes)
			{
				if (IsParticleProbeOnly(InParticle))
				{
					return true;
				}
			}

			return false;
		}

		//
		//
		//
		void DrawShapesImpl(const FGeometryParticleHandle* Particle, const FRigidTransform3& ShapeTransform, const FImplicitObject* Implicit, const FPerShapeData* Shape, const FReal Margin, const FColor& Color, const FRealSingle Duration, const FChaosDebugDrawSettings& Settings);


		void DrawShape(const FRigidTransform3& ShapeTransform, const FImplicitObject* Implicit, const FPerShapeData* Shape, const FColor& Color, const float Duration, const FChaosDebugDrawSettings* Settings)
		{
			DrawShapesImpl(nullptr, ShapeTransform, Implicit, Shape, 0.0f, Color, Duration, GetChaosDebugDrawSettings(Settings));
		}

		void DrawShapesConvexImpl(const TGeometryParticleHandle<FReal, 3>* Particle, const FRigidTransform3& ShapeTransform, const FConvex* Shape, const FReal InMargin, const FColor& Color, const FRealSingle Duration, const FChaosDebugDrawSettings& Settings)
		{
			if (Shape->HasStructureData())
			{
				const FReal Margin = InMargin + Shape->GetMargin();

				for (int32 PlaneIndex = 0; PlaneIndex < Shape->GetFaces().Num(); ++PlaneIndex)
				{
					const int32 PlaneVerticesNum = Shape->NumPlaneVertices(PlaneIndex);
					for (int32 PlaneVertexIndex = 0; PlaneVertexIndex < PlaneVerticesNum; ++PlaneVertexIndex)
					{
						const int32 VertexIndex0 = Shape->GetPlaneVertex(PlaneIndex, PlaneVertexIndex);
						const int32 VertexIndex1 = Shape->GetPlaneVertex(PlaneIndex, Utilities::WrapIndex(PlaneVertexIndex + 1, 0, PlaneVerticesNum));

						const FVec3 OuterP0 = ShapeTransform.TransformPosition(FVector(Shape->GetVertex(VertexIndex0)));
						const FVec3 OuterP1 = ShapeTransform.TransformPosition(FVector(Shape->GetVertex(VertexIndex1)));

						const FVec3 N0 = ShapeTransform.TransformVectorNoScale(Shape->GetPlane(PlaneIndex).Normal());
						const FVec3 ExplodeDelta = ChaosDebugDrawConvexExplodeDistance * N0;

						// Outer shape
						DrawDebugLine(OuterP0 + ExplodeDelta, OuterP1 + ExplodeDelta, Color, false, Duration, uint8(Settings.DrawPriority), Settings.ShapeThicknesScale * Settings.LineThickness);

						// Core shape and lines connecting core to outer
						if (Margin > 0.0f)
						{
							const FRealSingle LineThickness = 0.5f * Settings.ShapeThicknesScale * Settings.LineThickness;
							const FVec3 InnerP0 = ShapeTransform.TransformPositionNoScale(Shape->GetMarginAdjustedVertexScaled(VertexIndex0, Margin, ShapeTransform.GetScale3D(), nullptr));
							const FVec3 InnerP1 = ShapeTransform.TransformPositionNoScale(Shape->GetMarginAdjustedVertexScaled(VertexIndex1, Margin, ShapeTransform.GetScale3D(), nullptr));
							DrawDebugLine(InnerP0, InnerP1, FColor::Blue, false, Duration, uint8(Settings.DrawPriority), LineThickness);
							DrawDebugLine(InnerP0, OuterP0, FColor::Black, false, Duration, uint8(Settings.DrawPriority), LineThickness);
						}

						// Vertex and face normal
						if (bChaosDebugDebugDrawConvexVertices)
						{
							DrawDebugLine(OuterP0 + ExplodeDelta, OuterP0 + ExplodeDelta + Settings.DrawScale * 20.0f * N0, FColor::Green, false, Duration, uint8(Settings.DrawPriority), Settings.LineThickness);
						}
					}
				}
			}
		}

		void DrawShapesHeightFieldImpl(const TGeometryParticleHandle<FReal, 3>* Particle, const FRigidTransform3& ShapeTransform, const FHeightField* Shape, const FColor& Color, const FRealSingle Duration, const FChaosDebugDrawSettings& Settings)
		{
			const FVec3& WorldQueryCenter = GetCenterOfInterest();
			const FReal WorldQueryRadius = GetRadiusOfInterest();
			const FAABB3 WorldQueryBounds = FAABB3(WorldQueryCenter - FVec3(WorldQueryRadius), WorldQueryCenter + FVec3(WorldQueryRadius));
			const FAABB3 LocalQueryBounds = WorldQueryBounds.InverseTransformedAABB(ShapeTransform);

			Shape->VisitTriangles(LocalQueryBounds, ShapeTransform, [&](const FTriangle& Tri, const int32 TriangleIndex, const int32 VertexIndex0, const int32 VertexIndex1, const int32 VertexIndex2)
			{
				const FVec3& A = Tri[0];
				const FVec3& B = Tri[1];
				const FVec3& C = Tri[2];
				DrawDebugLine(A, B, Color, false, Duration, 0, Settings.ShapeThicknesScale * Settings.LineThickness);
				DrawDebugLine(B, C, Color, false, Duration, 0, Settings.ShapeThicknesScale * Settings.LineThickness);
				DrawDebugLine(C, A, Color, false, Duration, 0, Settings.ShapeThicknesScale * Settings.LineThickness);
			});
		}

		void DrawShapesTriangleMeshImpl(const TGeometryParticleHandle<FReal, 3>* Particle, const FRigidTransform3& ShapeTransform, const FTriangleMeshImplicitObject* Shape, const FColor& Color, const FRealSingle Duration, const FChaosDebugDrawSettings& Settings)
		{
			const FVec3& WorldQueryCenter = GetCenterOfInterest();
			const FReal WorldQueryRadius = GetRadiusOfInterest();
			const FAABB3 WorldQueryBounds = FAABB3(WorldQueryCenter - FVec3(WorldQueryRadius), WorldQueryCenter + FVec3(WorldQueryRadius));
			const FAABB3 LocalQueryBounds = WorldQueryBounds.InverseTransformedAABB(ShapeTransform);

			Shape->VisitTriangles(LocalQueryBounds, ShapeTransform, [&](const FTriangle& Tri, const int32 TriangleIndex, const int32 VertexIndex0, const int32 VertexIndex1, const int32 VertexIndex2)
			{
				const FVec3& A = Tri[0];
				const FVec3& B = Tri[1];
				const FVec3& C = Tri[2];
				DrawDebugLine(A, B, Color, false, Duration, 0, Settings.ShapeThicknesScale * Settings.LineThickness);
				DrawDebugLine(B, C, Color, false, Duration, 0, Settings.ShapeThicknesScale * Settings.LineThickness);
				DrawDebugLine(C, A, Color, false, Duration, 0, Settings.ShapeThicknesScale * Settings.LineThickness);
			});
		}

		void DrawShapesLevelSetImpl(const TGeometryParticleHandle<FReal, 3>* Particle, const FRigidTransform3& ShapeTransform, const FLevelSet* Shape, const FColor& Color, const FRealSingle Duration, const FChaosDebugDrawSettings& Settings)
		{
			if (!Settings.bShowLevelSetCollision)
			{
				return;
			}

			if (Particle == nullptr)
			{
				return;
			}

			if (const TPBDRigidParticleHandle<FReal, 3>* Rigid = Particle->CastToRigidParticle())
			{
				const TUniquePtr<TBVHParticles<FReal, 3>>& CollisionParticles = Rigid->CollisionParticles();
				if (CollisionParticles != nullptr)
				{
					for (int32 ParticleIndex = 0; ParticleIndex < (int32)CollisionParticles->Size(); ++ParticleIndex)
					{
						const FVec3 P = ShapeTransform.TransformPosition(CollisionParticles->GetX(ParticleIndex));
						DrawDebugPoint(P, Color, false, Duration, 0, Settings.PointSize);
					}
				}
			}
		}

		template <bool bInstanced>
		void DrawShapesScaledImpl(const FGeometryParticleHandle* Particle, const FRigidTransform3& ShapeTransform, const FImplicitObject* Implicit, const FPerShapeData* Shape, const FReal Margin, const FColor& Color, const FRealSingle Duration, const FChaosDebugDrawSettings& Settings)
		{
			const EImplicitObjectType PackedType = Implicit->GetType();
			const EImplicitObjectType InnerType = GetInnerType(PackedType);
			CHAOS_CHECK(IsScaled(PackedType));
			CHAOS_CHECK(IsInstanced(PackedType) == bInstanced);

			FRigidTransform3 ScaleTM = FRigidTransform3::Identity;
			switch (InnerType)
			{
			case ImplicitObjectType::Sphere:
				break;
			case ImplicitObjectType::Box:
				break;
			case ImplicitObjectType::Plane:
				break;
			case ImplicitObjectType::Capsule:
				break;
			case ImplicitObjectType::Transformed:
				break;
			case ImplicitObjectType::Union:
				break;
			case ImplicitObjectType::LevelSet:
			{
				const TImplicitObjectScaled<FLevelSet, bInstanced>* Scaled = Implicit->template GetObject<TImplicitObjectScaled<FLevelSet, bInstanced>>();
				// even though thhe levelset is scaled, the debugdraw uses the collisionParticles  that are pre-scaled
				// so no need to pass the scaled transform and just extract the wrapped LevelSet
				DrawShapesImpl(Particle, ShapeTransform, Scaled->GetUnscaledObject(), Shape, Scaled->GetMarginf(), Color, Duration, Settings);
				break;
			}
			case ImplicitObjectType::Unknown:
				break;
			case ImplicitObjectType::Convex:
			{
				const TImplicitObjectScaled<FConvex, bInstanced>* Scaled = Implicit->template GetObject<TImplicitObjectScaled<FConvex, bInstanced>>();
				ScaleTM.SetScale3D(Scaled->GetScale());
				DrawShapesImpl(Particle, ScaleTM * ShapeTransform, Scaled->GetUnscaledObject(), Shape, Scaled->GetMarginf(), Color, Duration, Settings);
				break;
			}
			case ImplicitObjectType::TaperedCylinder:
				break;
			case ImplicitObjectType::Cylinder:
				break;
			case ImplicitObjectType::TriangleMesh:
			{
				const TImplicitObjectScaled<FTriangleMeshImplicitObject, bInstanced>* Scaled = Implicit->template GetObject<TImplicitObjectScaled<FTriangleMeshImplicitObject, bInstanced>>();
				ScaleTM.SetScale3D(Scaled->GetScale());
				DrawShapesImpl(Particle, ScaleTM * ShapeTransform, Scaled->GetUnscaledObject(), Shape, 0.0f, Color, Duration, Settings);
				break;
			}
			case ImplicitObjectType::HeightField:
			{
				const TImplicitObjectScaled<FHeightField, bInstanced>* Scaled = Implicit->template GetObject<TImplicitObjectScaled<FHeightField, bInstanced>>();
				ScaleTM.SetScale3D(Scaled->GetScale());
				DrawShapesImpl(Particle, ScaleTM * ShapeTransform, Scaled->GetUnscaledObject(), Shape, 0.0f, Color, Duration, Settings);
				break;
			}
			default:
				break;
			}
		}

		void DrawShapesInstancedImpl(const FGeometryParticleHandle* Particle, const FRigidTransform3& ShapeTransform, const FImplicitObject* Implicit, const FPerShapeData* Shape, const FReal Margin, const FColor& Color, const FRealSingle Duration, const FChaosDebugDrawSettings& Settings)
		{
			const EImplicitObjectType PackedType = Implicit->GetType();
			const EImplicitObjectType InnerType = GetInnerType(PackedType);
			CHAOS_CHECK(IsScaled(PackedType) == false);
			CHAOS_CHECK(IsInstanced(PackedType));

			switch (InnerType)
			{
			case ImplicitObjectType::Sphere:
				break;
			case ImplicitObjectType::Box:
				break;
			case ImplicitObjectType::Plane:
				break;
			case ImplicitObjectType::Capsule:
				break;
			case ImplicitObjectType::Transformed:
				break;
			case ImplicitObjectType::Union:
				break;
			case ImplicitObjectType::LevelSet:
				break;
			case ImplicitObjectType::Unknown:
				break;
			case ImplicitObjectType::Convex:
			{
				const TImplicitObjectInstanced<FConvex>* Instanced = Implicit->template GetObject<TImplicitObjectInstanced<FConvex>>();
				DrawShapesImpl(Particle, ShapeTransform, Instanced->GetInstancedObject(), Shape, Instanced->GetMarginf(), Color, Duration, Settings);
				break;
			}
			case ImplicitObjectType::TaperedCylinder:
				break;
			case ImplicitObjectType::Cylinder:
				break;
			case ImplicitObjectType::TriangleMesh:
			{
				const TImplicitObjectInstanced<FTriangleMeshImplicitObject>* Scaled = Implicit->template GetObject<TImplicitObjectInstanced<FTriangleMeshImplicitObject>>();
				DrawShapesImpl(Particle, ShapeTransform, Scaled->GetInstancedObject(), Shape, 0.0f, Color, Duration, Settings);
				break;
			}
			case ImplicitObjectType::HeightField:
			{
				const TImplicitObjectInstanced<FHeightField>* Scaled = Implicit->template GetObject<TImplicitObjectInstanced<FHeightField>>();
				DrawShapesImpl(Particle, ShapeTransform, Scaled->GetInstancedObject(), Shape, 0.0f, Color, Duration, Settings);
				break;
			}
			default:
				break;
			}
		}

		void DrawShapesImpl(const FGeometryParticleHandle* Particle, const FRigidTransform3& ShapeTransform, const FImplicitObject* Implicit, const FPerShapeData* Shape, const FReal Margin, const FColor& Color, const FRealSingle Duration, const FChaosDebugDrawSettings& Settings)
		{
			if (Implicit == nullptr)
			{
				return;
			}

			const EImplicitObjectType PackedType = Implicit->GetType(); // Type includes scaling and instancing data
			const EImplicitObjectType InnerType = GetInnerType(Implicit->GetType());

			// Are we within the region of interest?
			if (Implicit->HasBoundingBox() && !IsInRegionOfInterest(Implicit->BoundingBox().TransformedAABB(ShapeTransform)))
			{
				return;
			}

			// Unwrap the wrapper/aggregating shapes
			if (IsScaled(PackedType))
			{
				if (IsInstanced(PackedType))
				{
					DrawShapesScaledImpl<true>(Particle, ShapeTransform, Implicit, Shape, Margin, Color, Duration, Settings);
				}
				else
				{
					DrawShapesScaledImpl<false>(Particle, ShapeTransform, Implicit, Shape, Margin, Color, Duration, Settings);
				}
				return;
			}
			else if (IsInstanced(PackedType))
			{
				DrawShapesInstancedImpl(Particle, ShapeTransform, Implicit, Shape, Margin, Color, Duration, Settings);
				return;
			}
			else if (InnerType == ImplicitObjectType::Transformed)
			{
				const TImplicitObjectTransformed<FReal, 3>* Transformed = Implicit->template GetObject<TImplicitObjectTransformed<FReal, 3>>();
				FRigidTransform3 TransformedTransform = FRigidTransform3(ShapeTransform.TransformPosition(Transformed->GetTransform().GetLocation()), ShapeTransform.GetRotation() * Transformed->GetTransform().GetRotation());
				DrawShapesImpl(Particle, TransformedTransform, Transformed->GetTransformedObject(), Shape, Margin, Color, Duration, Settings);
				return;
			}
			else if ((InnerType == ImplicitObjectType::Union) || (InnerType == ImplicitObjectType::UnionClustered))
			{
				if(InnerType == ImplicitObjectType::Union)
				{
					const FImplicitObjectUnion* Union = Implicit->template GetObject<FImplicitObjectUnion>();
					for (int32 UnionIdx = 0; UnionIdx < Union->GetObjects().Num(); ++UnionIdx)
					{
						DrawShapesImpl(Particle, ShapeTransform, Union->GetObjects()[UnionIdx].GetReference(), Shape, Margin, Color, Duration, Settings);
					}
				}
				else
				{
					// For union clustered if we have simplified convexes we are generating the particle
					// shapes since we have more than children. Otherwise we can grab the shape from the original particle lookup
					const FImplicitObjectUnionClustered* Union = Implicit->template GetObject<FImplicitObjectUnionClustered>();
					for (auto& UnionImplicit : Union->GetObjects())
					{
						const TPBDRigidParticleHandle<FReal, 3>* OriginalParticle = Union->FindParticleForImplicitObject(UnionImplicit.GetReference());
						if (ensure(OriginalParticle))
						{
							DrawShapesImpl(Particle, ShapeTransform, UnionImplicit.GetReference(), Shape, Margin, Color, Duration, Settings);
						}
					}
				}
				return;
			}

			// Whether we should show meshes and non-mesh shapes
			bool bShowMeshes = Settings.bShowComplexCollision;
			bool bShowNonMeshes = Settings.bShowSimpleCollision;

			if (Shape != nullptr)
			{
				bShowMeshes = (Settings.bShowComplexCollision && (Shape->GetCollisionTraceType() != EChaosCollisionTraceFlag::Chaos_CTF_UseSimpleAsComplex))
					|| (Settings.bShowSimpleCollision && (Shape->GetCollisionTraceType() == EChaosCollisionTraceFlag::Chaos_CTF_UseComplexAsSimple));
				bShowNonMeshes = (Settings.bShowSimpleCollision && (Shape->GetCollisionTraceType() != EChaosCollisionTraceFlag::Chaos_CTF_UseComplexAsSimple))
					|| (Settings.bShowComplexCollision && (Shape->GetCollisionTraceType() == EChaosCollisionTraceFlag::Chaos_CTF_UseSimpleAsComplex));
			}

			// Quit if we don't want to show this shape
			const bool bIsMesh = (InnerType == ImplicitObjectType::TriangleMesh);
			if (bIsMesh && !bShowMeshes)
			{
				return;
			}
			else if (!bIsMesh && !bShowNonMeshes)
			{
				return;
			}

			FColor ShapeColor = Color;
			if (Particle != nullptr)
			{
				if (bChaosDebugDebugDrawColorShapesByShapeType)
				{
					ShapeColor = Settings.ShapesColorsPerShapeType.GetColorFromShapeType(InnerType);
				}
				if (bChaosDebugDebugDrawColorShapesByIsland)
				{
					const int32 IslandId = (Particle->GetConstraintGraphNode() != nullptr) ? Particle->GetConstraintGraphNode()->GetIslandId() : INDEX_NONE;
					ShapeColor = GetIslandColor(IslandId, true);
				}
				if (bChaosDebugDebugDrawColorShapesByInternalCluster)
				{
					ShapeColor = FColor::Black;
					if (const FPBDRigidClusteredParticleHandle* ClusteredParticle = Particle->CastToClustered())
					{
						if (bChaosDebugDebugDrawColorShapesByClusterUnion)
						{
							if (ClusteredParticle->PhysicsProxy()->GetType() == EPhysicsProxyType::ClusterUnionProxy)
							{
								ShapeColor = GetIndexColor(FMath::Abs(ClusteredParticle->ClusterGroupIndex()));
							}
						}
						else
						{
							if (ClusteredParticle->InternalCluster())
							{
								ShapeColor = FColor::Purple;
							}
						}
					}
				}
				if (bChaosDebugDebugDrawColorShapesByConvexType)
				{
					if(InnerType == ImplicitObjectType::Convex)
					{
						ShapeColor = FColor::Orange;
						if (const FPBDRigidClusteredParticleHandle* ClusteredParticle = Particle->CastToClustered())
						{
							if(ClusteredParticle->ConvexOptimizer() && ClusteredParticle->ConvexOptimizer()->IsValid())
							{
								ShapeColor = FColor::Green;
							}
						}
					}
					else
					{
						ShapeColor = FColor::Yellow;
					}
				}
			}
			if (Shape && !bChaosDebugDebugDrawShowQueryOnlyShapes)
			{
				if (!Shape->GetSimEnabled() && !Shape->GetIsProbe())
				{
					return;
				}
			}
			if (Shape && !bChaosDebugDebugDrawShowSimOnlyShapes)
			{
				if (!Shape->GetQueryEnabled() && !Shape->GetIsProbe())
				{
					return;
				}
			}
			if (Shape && !bChaosDebugDebugDrawShowProbeOnlyShapes)
			{
				if (!Shape->GetSimEnabled() && !Shape->GetQueryEnabled())
				{
					return;
				}
			}

			// If we get here, we have an actual shape to render
			switch (InnerType)
			{
			case ImplicitObjectType::Sphere:
			{
				const FSphere* Sphere = Implicit->template GetObject<FSphere>();
				const FVec3 P = ShapeTransform.TransformPosition(FVec3(Sphere->GetCenterf()));
				const FVec3 Scale = ShapeTransform.GetScale3D();
				DrawDebugSphere(P, Sphere->GetRadiusf() * Scale.Z, 8, ShapeColor, false, Duration, uint8(Settings.DrawPriority), Settings.ShapeThicknesScale * Settings.LineThickness);
				break;
			}
			case ImplicitObjectType::Box:
			{
				const TBox<FReal, 3>* Box = Implicit->template GetObject<TBox<FReal, 3>>();
				const FVec3 P = ShapeTransform.TransformPosition(Box->GetCenter());
				const FVec3 Scale = ShapeTransform.GetScale3D();
				DrawDebugBox(P, (FReal)0.5 * Box->Extents() * Scale, ShapeTransform.GetRotation(), ShapeColor, false, Duration, uint8(Settings.DrawPriority), Settings.ShapeThicknesScale * Settings.LineThickness);
				break;
			}
			case ImplicitObjectType::Plane:
				break;
			case ImplicitObjectType::Capsule:
			{
				const FCapsule* Capsule = Implicit->template GetObject<FCapsule>();
				const FVec3 P = ShapeTransform.TransformPosition(FVec3(Capsule->GetCenterf()));
				const FRotation3 Q = ShapeTransform.GetRotation() * FRotationMatrix::MakeFromZ(Capsule->GetAxis());
				const FVec3 Scale = ShapeTransform.GetScale3D();
				DrawDebugCapsule(P, ((FReal)0.5 * Capsule->GetHeightf() + Capsule->GetRadiusf()) * Scale.Z, Capsule->GetRadiusf() * Scale.X, Q, ShapeColor, false, Duration, uint8(Settings.DrawPriority), Settings.ShapeThicknesScale * Settings.LineThickness);
				break;
			}
			case ImplicitObjectType::LevelSet:
			{
				const FLevelSet* LevelSet = Implicit->template GetObject<FLevelSet>();
				DrawShapesLevelSetImpl(Particle, ShapeTransform, LevelSet, ShapeColor, Duration, Settings);
				break;
			}
			case ImplicitObjectType::Convex:
			{
				const FConvex* Convex = Implicit->template GetObject<FConvex>();

				const FReal NetMargin = bChaosDebugDebugDrawCoreShapes ? Margin + Convex->GetMargin() : 0.0f;
				DrawShapesConvexImpl(Particle, ShapeTransform, Convex, NetMargin, ShapeColor, Duration, Settings);

				// Generate the exact marging-reduced convex for comparison with the runtime approximation
				// Warning: extremely expensive!
				if (bChaosDebugDebugDrawExactCoreShapes)
				{
					TArray<FConvex::FVec3Type> ScaledVerts(Convex->GetVertices());
					FConvex::FVec3Type Scale(ShapeTransform.GetScale3D());
					for (FConvex::FVec3Type& Vert : ScaledVerts)
					{
						Vert *= Scale;
					}
					FConvex ShrunkScaledConvex(ScaledVerts, FReal(0));
					ShrunkScaledConvex.MovePlanesAndRebuild(FConvex::FRealType(-Margin)); // potential loss of precision but margin should remain within the float range

					const FRigidTransform3 ShrunkScaledTransform = FRigidTransform3(ShapeTransform.GetTranslation(), ShapeTransform.GetRotation());
					DrawShapesConvexImpl(Particle, ShrunkScaledTransform, &ShrunkScaledConvex, 0.0f, FColor::Red, Duration, Settings);
				}
				break;
			}
			case ImplicitObjectType::TaperedCylinder:
				break;
			case ImplicitObjectType::Cylinder:
				break;
			case ImplicitObjectType::TriangleMesh:
			{
				const FTriangleMeshImplicitObject* TriangleMesh = Implicit->template GetObject<FTriangleMeshImplicitObject>();
				DrawShapesTriangleMeshImpl(Particle, ShapeTransform, TriangleMesh, ShapeColor, Duration, Settings);
				break;
			}
			case ImplicitObjectType::HeightField:
			{
				const FHeightField* HeightField = Implicit->template GetObject<FHeightField>();
				DrawShapesHeightFieldImpl(Particle, ShapeTransform, HeightField, ShapeColor, Duration, Settings);
				break;
			}
			default:
				break;
			}

			if (bChaosDebugDebugDrawCollisionParticles && (Particle != nullptr))
			{
				if (const TPBDRigidParticleHandle<FReal, 3>* Rigid = Particle->CastToRigidParticle())
				{
					const TUniquePtr<FBVHParticles>& Particles = Rigid->CollisionParticles();
					if (Particles != nullptr)
					{
						for (int32 ParticleIndex = 0; ParticleIndex < (int32)Particles->Size(); ++ParticleIndex)
						{
							FVec3 P = ShapeTransform.TransformPosition(Particles->GetX(ParticleIndex));
							DrawDebugPoint(P, ShapeColor, false, Duration, uint8(Settings.DrawPriority), Settings.PointSize);
						}
					}
				}
			}

			if (bChaosDebugDebugDrawShapeBounds)
			{
				const FColor ShapeBoundsColor = FColor::Orange;
				const FAABB3& ShapeBounds = Implicit->BoundingBox();
				const FVec3 ShapeBoundsPos = ShapeTransform.TransformPosition(ShapeBounds.Center());
				DrawDebugBox(ShapeBoundsPos, 0.5f * ShapeBounds.Extents(), ShapeTransform.GetRotation(), ShapeBoundsColor, false, Duration, uint8(Settings.DrawPriority), Settings.LineThickness);
			}
		}
		
		bool DrawConvexOptimizerShapes(const FVec3& P, const FRotation3& Q, const FGeometryParticleHandle* Particle, const FColor& InColor, const FChaosDebugDrawSettings& Settings)
		{
			if (!bChaosDebugDebugDrawShowOptimizedConvexes)
			{
				return false;
			}

			if(auto ClusteredParticle = Particle->CastToClustered())
			{
				const TPimplPtr<Private::FConvexOptimizer>& ConvexOptimizer = ClusteredParticle->ConvexOptimizer();
				if(ConvexOptimizer && ConvexOptimizer->IsValid())
				{
					ConvexOptimizer->VisitCollisionObjects([&Particle, &P, &Q, &ConvexOptimizer, &InColor, &Settings](
						const FImplicitObject* ImplicitObject, const FRigidTransform3& RelativeTransform, const int32 RootObjectIndex, const int32 ObjectIndex, const int32 LeafObjectIndex)-> void
					{
						if(RootObjectIndex == INDEX_NONE && !ConvexOptimizer->GetShapeInstances().IsEmpty())
						{
							DrawShapesImpl(Particle, RelativeTransform * FRigidTransform3(P, Q), ImplicitObject, ConvexOptimizer->GetShapeInstances()[0].Get(), 0.0f, InColor, 0.0f, Settings);
						}
						else if(Particle->ShapeInstances().IsValidIndex(RootObjectIndex))
						{
							DrawShapesImpl(Particle, RelativeTransform * FRigidTransform3(P, Q), ImplicitObject, Particle->ShapeInstances()[RootObjectIndex].Get(), 0.0f, InColor, 0.0f, Settings);
						}
					});
					return true;
				}
			}
			return false;
		}

		void DrawParticleShapesImpl(const FRigidTransform3& SpaceTransform, const FGeometryParticleHandle* Particle, const FRigidTransform3& ParticleTransform, const FColor& InColor, const FChaosDebugDrawSettings& Settings)
		{
			const FRigidTransform3 ParticleSpaceTransform = ParticleTransform * SpaceTransform;

			if(!DrawConvexOptimizerShapes(ParticleTransform.GetTranslation(), ParticleTransform.GetRotation(), Particle, InColor, Settings))
			{
				for (const FShapeInstancePtr& ShapeInstance : Particle->ShapeInstances())
				{
					DrawShapesImpl(Particle, ParticleSpaceTransform, ShapeInstance->GetGeometry(), ShapeInstance.Get(), 0.0f, InColor, 0.0f, Settings);
				}
			}
		}

		void DrawBVHImpl(const FGeometryParticleHandle* Particle, const FRigidTransform3& ShapeTransform, const Private::FImplicitBVH* BVH, const FColor& UnusedColor, const FRealSingle Duration, const FChaosDebugDrawSettings& Settings)
		{
			if (ShouldIgnoreParticle(Particle))
			{
				return;
			}

			const FReal BoundsDrawShrink = 1.0f;	// 1cm
			int32 LeafIndex = 0;

			const auto& NodeVisitor = 
				[Particle, &ShapeTransform, BVH, &LeafIndex, BoundsDrawShrink, &Duration, &Settings](const FAABB3f& NodeBounds, const int32 NodeDepth, const int32 NodeIndex) -> bool
				{
					// Draw the node if it is at the level we are asked to draw, or it is a leaf and we want to draw leaves.
					if ((NodeDepth == ChaosDebugDebugDrawBVHLevel) || (BVH->NodeIsLeaf(NodeIndex) && ((ChaosDebugDebugDrawBVHLevel == INDEX_NONE) || (ChaosDebugDebugDrawBVHLevel > NodeDepth))))
					{
						const FColor Color = GetIndexColor(LeafIndex++);

						if (bChaosDebugDebugDrawBVHBounds)
						{
							const FVec3 NodeBoundsPos = ShapeTransform.TransformPosition(FVec3(NodeBounds.Center()));
							const FVec3 NodeBoundsExtents = 0.5f * FVec3(NodeBounds.Extents()) - FVec3(BoundsDrawShrink);
							DrawDebugBox(NodeBoundsPos, NodeBoundsExtents, ShapeTransform.GetRotation(), Color, false, Duration, uint8(Settings.DrawPriority), Settings.LineThickness);
						}

						if (bChaosDebugDebugDrawBVHShapes)
						{
							BVH->VisitNodeObjects(NodeIndex, 
								[Particle, &ShapeTransform, &Color, Duration, &Settings](const FImplicitObject * Implicit, const FRigidTransform3f& RelativeTransformf, const FAABB3f& RelativeBoundsf, const int32 RootObjectIndex, const int32 LeafObjectIndex) -> void
								{
									const int32 ShapeIndex = (RootObjectIndex != INDEX_NONE) ? RootObjectIndex : 0;
									const FShapeInstance* Shape = Particle->ShapeInstances()[ShapeIndex].Get();
									DrawShapesImpl(Particle, FRigidTransform3(RelativeTransformf) * ShapeTransform, Implicit, Shape, 0, Color, Duration, Settings);
								});
						}
					}
					return true;
				};

			BVH->VisitNodes(NodeVisitor);
		}

		void DrawParticleBVHImpl(const FRigidTransform3& SpaceTransform, const FGeometryParticleHandle* Particle, const FColor& InColor, const FChaosDebugDrawSettings& Settings)
		{
			if (const FImplicitObjectUnion* Union = Particle->GetGeometry()->template AsA<FImplicitObjectUnion>())
			{
				if (Union->GetBVH() != nullptr)
				{
					FVec3 P = SpaceTransform.TransformPosition(Particle->GetX());
					FRotation3 Q = SpaceTransform.GetRotation() * (Particle->GetR());

					DrawBVHImpl(Particle->Handle(), FRigidTransform3(P, Q), Union->GetBVH(), InColor, 0.0f, Settings);
				}
			}
		}

		static EImplicitObjectType GetFirstConcreteShapeType(const FImplicitObject* Shape)
		{
			EImplicitObjectType InnerType = GetInnerType(Shape->GetType());
			if (InnerType == ImplicitObjectType::Union)
			{
				const FImplicitObjectUnion* Union = Shape->template GetObject<FImplicitObjectUnion>();
				if (const TArray<Chaos::FImplicitObjectPtr>& UnionObjects = Union->GetObjects();  UnionObjects.Num() > 0)
				{
					// use the first as reference as we can only display one color for the bounds
					return GetFirstConcreteShapeType(UnionObjects[0].GetReference());
				}
			}
			else if (InnerType == ImplicitObjectType::Transformed)
			{
				const TImplicitObjectTransformed<FReal, 3>* Transformed = Shape->template GetObject<TImplicitObjectTransformed<FReal, 3>>();
				return GetFirstConcreteShapeType(Transformed->GetTransformedObject());
			}
			else if (InnerType == ImplicitObjectType::UnionClustered)
			{
				const FImplicitObjectUnionClustered* Union = Shape->template GetObject<FImplicitObjectUnionClustered>();
				if (const TArray<Chaos::FImplicitObjectPtr>& UnionObjects = Union->GetObjects();  UnionObjects.Num() > 0)
				{
					// use the first as reference as we can only display one color for the bounds
					return GetFirstConcreteShapeType(UnionObjects[0].GetReference());
				}
			}
			return InnerType;
		}

		void DrawParticleBoundsImpl(const FRigidTransform3& SpaceTransform, const FGeometryParticleHandle* InParticle, const FReal Dt, const FChaosDebugDrawSettings& Settings)
		{
			if (ShouldIgnoreParticle(InParticle))
			{
				return;
			}

			FConstGenericParticleHandle Particle = InParticle;

			const FAABB3 Box = InParticle->WorldSpaceInflatedBounds();

			const FVec3 P = SpaceTransform.TransformPosition(Box.GetCenter());
			const FRotation3 Q = SpaceTransform.GetRotation();
			FColor Color = Settings.BoundsColorsPerState.GetColorFromState(InParticle->ObjectState(), ParticleIsOneWayInteraction(InParticle));
			if (bChaosDebugDebugDrawColorBoundsByShapeType)
			{
				if (const FImplicitObject* Shape = Particle->GetGeometry())
				{
					Color = Settings.BoundsColorsPerShapeType.GetColorFromShapeType(GetFirstConcreteShapeType(Shape));
				}
			}

			DrawDebugBox(P, 0.5f * Box.Extents(), Q, Color, false, UE_KINDA_SMALL_NUMBER, uint8(Settings.DrawPriority), Settings.LineThickness);

			for (const auto& Shape : InParticle->ShapesArray())
			{
				const EImplicitObjectType ShapeType = GetFirstConcreteShapeType(Shape->GetGeometry());
				const bool bIsComplex = (ShapeType == ImplicitObjectType::TriangleMesh) || (ShapeType == ImplicitObjectType::HeightField);
				if (!bIsComplex)
				{
					const FAABB3 ShapeBox = Shape->GetWorldSpaceShapeBounds();
					const FVec3 ShapeP = SpaceTransform.TransformPosition(ShapeBox.GetCenter());
					const FRotation3 ShapeQ = SpaceTransform.GetRotation();
					const FColor ShapeColor = (bChaosDebugDebugDrawColorBoundsByShapeType) ? Settings.BoundsColorsPerShapeType.GetColorFromShapeType(ShapeType) : Color;

					DrawDebugBox(ShapeP, 0.5f * ShapeBox.Extents(), ShapeQ, ShapeColor, false, UE_KINDA_SMALL_NUMBER, uint8(Settings.DrawPriority), Settings.LineThickness);
				}
			}
		}

		void DrawParticleTransformImpl(const FRigidTransform3& SpaceTransform, const FGeometryParticleHandle* InParticle, int32 Index, FRealSingle ColorScale, const FChaosDebugDrawSettings& Settings)
		{
			if (ShouldIgnoreParticle(InParticle))
			{
				return;
			}

			const TPBDRigidParticleHandle<FReal, 3>* Rigid = InParticle->CastToRigidParticle();
			if (Rigid && Rigid->Disabled())
			{
				return;
			}
			
			FColor Red = (ColorScale * FColor::Red).ToFColor(false);
			FColor Green = (ColorScale * FColor::Green).ToFColor(false);
			FColor Blue = (ColorScale * FColor::Blue).ToFColor(false);

			FConstGenericParticleHandle Particle(InParticle);

			FRigidTransform3 SpaceCOM;
			if (Rigid != nullptr)
			{
				SpaceCOM = Rigid->GetTransformPQCom() * SpaceTransform;
			}
			else
			{
				SpaceCOM = InParticle->GetTransformXR() * SpaceTransform;
			}
			FVec3 PCOM = SpaceCOM.GetLocation();
			FRotation3 QCOM = SpaceCOM.GetRotation();
			FMatrix33 QCOMm = QCOM.ToMatrix();
			DrawDebugDirectionalArrow(PCOM, PCOM + Settings.DrawScale * Settings.BodyAxisLen * QCOMm.GetAxis(0), Settings.DrawScale * Settings.ArrowSize, Red, false, UE_KINDA_SMALL_NUMBER, uint8(Settings.DrawPriority), Settings.LineThickness);
			DrawDebugDirectionalArrow(PCOM, PCOM + Settings.DrawScale * Settings.BodyAxisLen * QCOMm.GetAxis(1), Settings.DrawScale * Settings.ArrowSize, Green, false, UE_KINDA_SMALL_NUMBER, uint8(Settings.DrawPriority), Settings.LineThickness);
			DrawDebugDirectionalArrow(PCOM, PCOM + Settings.DrawScale * Settings.BodyAxisLen * QCOMm.GetAxis(2), Settings.DrawScale * Settings.ArrowSize, Blue, false, UE_KINDA_SMALL_NUMBER, uint8(Settings.DrawPriority), Settings.LineThickness);

			FColor Black = FColor::Black;
			FColor Grey = (ColorScale * FColor(64, 64, 64)).ToFColor(false);
			FVec3 PActor = SpaceTransform.TransformPosition(FParticleUtilities::GetActorWorldTransform(Particle).GetTranslation());
			DrawDebugPoint(PActor, Black, false, UE_KINDA_SMALL_NUMBER, uint8(Settings.DrawPriority), Settings.DrawScale * Settings.PointSize);
			DrawDebugLine(PCOM, PActor, Grey, false, UE_KINDA_SMALL_NUMBER, uint8(Settings.DrawPriority), Settings.LineThickness);
		
			if (Index >= 0)
			{
				//DrawDebugString(PCOM + FontHeight * FVec3(0, 0, 1), FString::Format(TEXT("{0}{1}"), { Particle->IsKinematic()? TEXT("K"): TEXT("D"), Index }), nullptr, FColor::White, KINDA_SMALL_NUMBER, false, FontScale);
			}

			if ((Settings.VelScale > 0.0f) && (Particle->V().Size() > UE_KINDA_SMALL_NUMBER))
			{
				DrawDebugLine(PCOM, PCOM + SpaceTransform.TransformVector(Particle->V()) * Settings.VelScale, Red, false, UE_KINDA_SMALL_NUMBER, uint8(Settings.DrawPriority), Settings.LineThickness);
				//DrawDebugLine(PCOM, PCOM + SpaceTransform.TransformVector(Particle->VSmooth()) * Settings.VelScale, Blue, false, KINDA_SMALL_NUMBER, uint8(Settings.DrawPriority), Settings.LineThickness);
			}
			if ((Settings.AngVelScale > 0.0f) && (Particle->W().Size() > UE_KINDA_SMALL_NUMBER))
			{
				DrawDebugLine(PCOM, PCOM + SpaceTransform.TransformVector(Particle->W()) * Settings.AngVelScale, Green, false, UE_KINDA_SMALL_NUMBER, uint8(Settings.DrawPriority), Settings.LineThickness);
			}

			if (Settings.InertiaScale > 0.0f)
			{
				if (Rigid)
				{
					if (Rigid->InvM() != 0)
					{
						// Show the raw inertia in black
						if (!FVec3::IsNearlyEqual(Rigid->InvIConditioning(), TVec3<FRealSingle>(1), UE_SMALL_NUMBER))
						{
							FVec3 EquivalentBoxSize, EquivalentBoxCenter;
							if (Utilities::BoxFromInertia(Rigid->I(), Rigid->M(), EquivalentBoxCenter, EquivalentBoxSize))
							{
								const FVec3 BoxCoM = SpaceTransform.TransformPositionNoScale(PCOM + QCOM * EquivalentBoxCenter);
								DrawDebugBox(BoxCoM, 0.5f * Settings.InertiaScale * EquivalentBoxSize, QCOM, FColor::Black, false, 0.0f, 0, Settings.LineThickness);
							}
						}

						// Show the inertia used by the solver in magenta
						FVec3 EquivalentBoxConditionedSize, EquivalentBoxConditionedCenter;
						if (Utilities::BoxFromInertia(Rigid->ConditionedI(), Rigid->M(), EquivalentBoxConditionedCenter, EquivalentBoxConditionedSize))
						{
							const FVec3 BoxConditionedCoM = SpaceTransform.TransformPositionNoScale(PCOM + QCOM * EquivalentBoxConditionedCenter);
							DrawDebugBox(BoxConditionedCoM, 0.5f * Settings.InertiaScale * EquivalentBoxConditionedSize, QCOM, FColor::Magenta, false, 0.0f, 0, Settings.LineThickness);
						}
					}
				}
			}
		}

		void DrawCollisionImpl(const FRigidTransform3& SpaceTransform, const FPBDCollisionConstraint& Contact, FRealSingle ColorScale, const FRealSingle Duration, const FChaosDebugDrawSettings& Settings)
		{
			if (Contact.NumManifoldPoints() == 0)
			{
				return;
			}

			if ((Settings.ContactWidth > 0) || (Settings.ContactLen > 0) || (Settings.ContactInfoWidth > 0) || (Settings.ImpulseScale > 0.0f))
			{
				const FConstGenericParticleHandle Particle0 = Contact.GetParticle0();
				const FConstGenericParticleHandle Particle1 = Contact.GetParticle1();
				const FRigidTransform3 WorldActorTransform0 = FParticleUtilities::GetActorWorldTransform(Particle0);
				const FRigidTransform3 WorldActorTransform1 = FParticleUtilities::GetActorWorldTransform(Particle1);

				// Are we within the region of interest?
				if (Contact.GetParticle0()->HasBounds() && !IsInRegionOfInterest(Contact.GetParticle0()->WorldSpaceInflatedBounds().TransformedAABB(SpaceTransform)))
				{
					return;
				}
				if (Contact.GetParticle1()->HasBounds() && !IsInRegionOfInterest(Contact.GetParticle1()->WorldSpaceInflatedBounds().TransformedAABB(SpaceTransform)))
				{
					return;
				}

				for (int32 PointIndex = 0; PointIndex < Contact.NumManifoldPoints(); ++PointIndex)
				{
					const FManifoldPoint& ManifoldPoint = Contact.GetManifoldPoint(PointIndex);
					const FManifoldPointResult& ManifoldPointResult = Contact.GetManifoldPointResult(PointIndex);

					const bool bIsProbe = Contact.GetIsProbe();
					const bool bIsActive = ManifoldPointResult.bIsValid && (!ManifoldPointResult.NetPushOut.IsNearlyZero() || !ManifoldPointResult.NetImpulse.IsNearlyZero() || (!Contact.GetUseManifold() && !Contact.AccumulatedImpulse.IsNearlyZero()));
					const bool bPruned = ManifoldPoint.Flags.bDisabled;
					if ((bPruned || !bIsActive) && !bChaosDebugDebugDrawInactiveContacts)
					{
						continue;
					}

					const int32 ContactPlaneOwner = 1;
					const int32 ContactPointOwner = 1 - ContactPlaneOwner;
					const FRigidTransform3& PlaneTransform = (ContactPlaneOwner == 0) ? Contact.GetShapeRelativeTransform0() * WorldActorTransform0 : Contact.GetShapeRelativeTransform1() * WorldActorTransform1;
					const FRigidTransform3& PointTransform = (ContactPlaneOwner == 0) ? Contact.GetShapeRelativeTransform1() * WorldActorTransform1 : Contact.GetShapeRelativeTransform0() * WorldActorTransform0;
					const FConstGenericParticleHandle PlaneParticle = (ContactPlaneOwner == 0) ? Particle0 : Particle1;
					const FVec3 PlaneNormal = PlaneTransform.TransformVectorNoScale(FVec3(ManifoldPoint.ContactPoint.ShapeContactNormal));
					const FVec3 PointLocation = PointTransform.TransformPosition(FVec3(ManifoldPoint.ContactPoint.ShapeContactPoints[ContactPointOwner]));
					const FVec3 PlaneLocation = PlaneTransform.TransformPosition(FVec3(ManifoldPoint.ContactPoint.ShapeContactPoints[ContactPlaneOwner]));
					const FVec3 PointPlaneLocation = PointLocation - FVec3::DotProduct(PointLocation - PlaneLocation, PlaneNormal) * PlaneNormal;

					// Dynamic friction, restitution = red
					// Static friction, no restitution = green
					// Inactive = gray
					FColor DiscColor = FColor(250, 0, 0);
					FColor PhiDiscColor = FColor(128, 128, 0);
					FColor PlaneNormalColor = FColor(250, 0, 0);
					FColor EdgeNormalColor = FColor(250, 150, 0);
					FColor ImpulseColor = FColor(0, 0, 250);
					FColor PushOutColor = FColor(0, 250, 0);
					FColor PushOutImpusleColor = FColor(0, 250, 250);
					FReal ContactLenScale = FReal(1);
					if (ManifoldPointResult.bInsideStaticFrictionCone)
					{
						DiscColor = FColor(150, 200, 0);
						PhiDiscColor = FColor(150, 200, 0);
					}
					if (bIsProbe)
					{
						DiscColor = FColor(50, 180, 180);
						PhiDiscColor = FColor(50, 180, 180);
						PlaneNormalColor = FColor(50, 180, 180);
						EdgeNormalColor = FColor(50, 220, 180);
					}
					else if (!bIsActive)
					{
						DiscColor = FColor(150, 150, 150);
						PhiDiscColor = FColor(150, 150, 150);
						PlaneNormalColor = FColor(150, 150, 150);
						EdgeNormalColor = FColor(150, 180, 150);
						ContactLenScale = 0.75;
					}
					if (bPruned)
					{
						DiscColor = FColor(50, 50, 50);
						PhiDiscColor = FColor(50, 50, 50);
						PlaneNormalColor = FColor(50, 50, 50);
						EdgeNormalColor = FColor(50, 80, 50);
						ContactLenScale = 0.5;
					}

					const FVec3 WorldPointLocation = SpaceTransform.TransformPosition(PointLocation);
					const FVec3 WorldPlaneLocation = SpaceTransform.TransformPosition(PlaneLocation);
					const FVec3 WorldPointPlaneLocation = SpaceTransform.TransformPosition(PointPlaneLocation);
					const FVec3 WorldPlaneNormal = SpaceTransform.TransformVectorNoScale(PlaneNormal);
					const FMatrix Axes = FRotationMatrix::MakeFromX(WorldPlaneNormal);

					// Pushout
					if ((Settings.PushOutScale > 0) && ManifoldPointResult.bIsValid && !ManifoldPointResult.NetPushOut.IsNearlyZero())
					{
						FColor Color = PushOutImpusleColor;
						DrawDebugLine(WorldPointPlaneLocation, WorldPointPlaneLocation + Settings.DrawScale * Settings.PushOutScale * SpaceTransform.TransformVectorNoScale(FVec3(ManifoldPointResult.NetPushOut)), Color, false, Duration, uint8(Settings.DrawPriority), Settings.LineThickness);
					}
					if ((Settings.ImpulseScale > 0) && ManifoldPointResult.bIsValid && !ManifoldPointResult.NetImpulse.IsNearlyZero())
					{
						FColor Color = ImpulseColor;
						DrawDebugLine(WorldPointPlaneLocation, WorldPointPlaneLocation + Settings.DrawScale * Settings.ImpulseScale * SpaceTransform.TransformVectorNoScale(FVec3(ManifoldPointResult.NetImpulse)), Color, false, Duration, uint8(Settings.DrawPriority), Settings.LineThickness);
					}

					// Manifold point and normal
					if (Settings.ContactWidth > 0)
					{
						FColor C0 = DiscColor;
						DrawDebugCircle(WorldPlaneLocation, Settings.DrawScale * Settings.ContactWidth, 12, C0, false, Duration, uint8(Settings.DrawPriority), Settings.LineThickness, Axes, false);
						DrawDebugCircle(WorldPointLocation, 0.5f * Settings.DrawScale * Settings.ContactWidth, 12, C0, false, Duration, uint8(Settings.DrawPriority), Settings.LineThickness, Axes, false);
						if (ManifoldPoint.InitialPhi != 0)
						{
							FColor C3 = FColor::Silver;
							DrawDebugCircle(WorldPlaneLocation + ManifoldPoint.InitialPhi * WorldPlaneNormal, 0.25f * Settings.DrawScale * Settings.ContactWidth, 12, C3, false, Duration, uint8(Settings.DrawPriority), Settings.LineThickness, Axes, false);
						}
					}
					if (Settings.ContactLen > 0)
					{
						FColor NormalColor = ((ManifoldPoint.ContactPoint.ContactType != EContactPointType::EdgeEdge) ? PlaneNormalColor : EdgeNormalColor);
						FColor C1 = NormalColor;
						DrawDebugLine(WorldPlaneLocation, WorldPlaneLocation + Settings.DrawScale * Settings.ContactLen * ContactLenScale * WorldPlaneNormal, C1, false, Duration, uint8(Settings.DrawPriority), Settings.LineThickness);
					}

					// Whether restored
					if (Settings.ContactInfoWidth > 0)
					{
						if (Contact.WasManifoldRestored())
						{
							const FReal BoxScale = Settings.DrawScale * Settings.ContactInfoWidth;
							DrawDebugBox(WorldPlaneLocation, FVec3(BoxScale, BoxScale, FReal(0.01)), FRotation3(FRotationMatrix::MakeFromZ(WorldPlaneNormal)), FColor::Blue, false, Duration, uint8(Settings.DrawPriority), 0.5f * Settings.LineThickness);
						}
						else if (ManifoldPoint.Flags.bWasRestored)
						{
							const FReal BoxScale = Settings.DrawScale * Settings.ContactInfoWidth;
							DrawDebugBox(WorldPlaneLocation, FVec3(BoxScale, BoxScale, FReal(0.01)), FRotation3(FRotationMatrix::MakeFromZ(WorldPlaneNormal)), FColor::Purple, false, Duration, uint8(Settings.DrawPriority), 0.5f * Settings.LineThickness);
						}
						else if (ManifoldPoint.Flags.bWasReplaced)
						{
							const FReal BoxScale = Settings.DrawScale * Settings.ContactInfoWidth;
							DrawDebugBox(WorldPlaneLocation, FVec3(BoxScale, BoxScale, FReal(0.01)), FRotation3(FRotationMatrix::MakeFromZ(WorldPlaneNormal)), FColor::Orange, false, Duration, uint8(Settings.DrawPriority), 0.5f * Settings.LineThickness);
						}
					}

					// Sleeping
					if (Settings.ContactInfoWidth > 0)
					{
						if (Contact.IsSleeping())
						{
							const FReal BoxScale = Settings.DrawScale * Settings.ContactInfoWidth * 1.1f;
							DrawDebugBox(WorldPlaneLocation, FVec3(BoxScale, BoxScale, FReal(0.01)), FRotation3(FRotationMatrix::MakeFromZ(WorldPlaneNormal)), FColor::Black, false, Duration, uint8(Settings.DrawPriority), 0.5f * Settings.LineThickness);
						}
					}
				}

				// AccumulatedImpulse
				if ((Settings.ImpulseScale > 0) && !Contact.GetAccumulatedImpulse().IsNearlyZero())
				{
					FColor Color = FColor::White;
					const FVec3 ImpulsePos = SpaceTransform.TransformPosition(WorldActorTransform0.GetTranslation());
					DrawDebugLine(ImpulsePos, ImpulsePos + Settings.DrawScale * Settings.ImpulseScale * SpaceTransform.TransformVectorNoScale(Contact.GetAccumulatedImpulse()), Color, false, Duration, uint8(Settings.DrawPriority), Settings.LineThickness);
				}

				// Static Friction Anchors (bets viewed with DebugDrawPostIntegrateCollisions)
				if (bChaosDebugDrawCollisionAnchors)
				{
					for (int32 PointIndex = 0; PointIndex < Contact.NumSavedManifoldPoints(); ++PointIndex)
					{
						const FSavedManifoldPoint& SavedManifoldPoint = Contact.GetSavedManifoldPoint(PointIndex);
						const int32 ContactPlaneOwner = 1;
						const int32 ContactPointOwner = 1 - ContactPlaneOwner;
						const FRigidTransform3& PlaneTransform = (ContactPlaneOwner == 0) ? Contact.GetShapeRelativeTransform0() * WorldActorTransform0 : Contact.GetShapeRelativeTransform1() * WorldActorTransform1;
						const FRigidTransform3& PointTransform = (ContactPlaneOwner == 0) ? Contact.GetShapeRelativeTransform1() * WorldActorTransform1 : Contact.GetShapeRelativeTransform0() * WorldActorTransform0;
						const FVec3 PointLocation = PointTransform.TransformPosition(FVec3(SavedManifoldPoint.ShapeContactPoints[ContactPointOwner]));
						const FVec3 PlaneLocation = PlaneTransform.TransformPosition(FVec3(SavedManifoldPoint.ShapeContactPoints[ContactPlaneOwner]));
				
						FColor Color = FColor::Black;
						DrawDebugLine(PointLocation, PlaneLocation, Color, false, Duration, uint8(Settings.DrawPriority), Settings.LineThickness);
					}
				}

				// Show the sphere approximation if enabled
				//if ((CVars::ChaosOneWayInteractionPairCollisionMode == (int32)EOneWayInteractionPairCollisionMode::SphereCollision) && Particle0->OneWayInteraction() && Particle1->OneWayInteraction())
				//{
				//	DrawShape(Contact.GetShapeWorldTransform0(), Contact.GetImplicit0(), nullptr, FColor::Green, 0, &Settings);
				//	DrawShape(Contact.GetShapeWorldTransform1(), Contact.GetImplicit1(), nullptr, FColor::Green, 0, &Settings);
				//}
			}
			if (Settings.ContactOwnerWidth > 0)
			{
				const FVec3 Location = SpaceTransform.TransformPosition(Contact.CalculateWorldContactLocation());
				const FVec3 Normal = SpaceTransform.TransformVector(Contact.CalculateWorldContactNormal());

				const FColor C3 = FColor::White;
				const FMatrix Axes = FRotationMatrix::MakeFromX(Normal);
				const FVec3 P0 = SpaceTransform.TransformPosition(Contact.GetShape0()->GetWorldSpaceShapeBounds().GetCenter());
				const FVec3 P1 = SpaceTransform.TransformPosition(Contact.GetShape1()->GetWorldSpaceShapeBounds().GetCenter());
				DrawDebugLine(Location, P0, C3, false, UE_KINDA_SMALL_NUMBER, uint8(Settings.DrawPriority), Settings.LineThickness * 0.5f);
				DrawDebugLine(Location, P1, C3, false, UE_KINDA_SMALL_NUMBER, uint8(Settings.DrawPriority), Settings.LineThickness * 0.5f);
			}
		}
		
		void DrawCollisionImpl(const FRigidTransform3& SpaceTransform, const FPBDCollisionConstraintHandle* ConstraintHandle, FRealSingle ColorScale, const FRealSingle Duration, const FChaosDebugDrawSettings& Settings)
		{
			DrawCollisionImpl(SpaceTransform, ConstraintHandle->GetContact(), ColorScale, Duration, Settings);
		}

		void DrawParticleCCDCollisionImpulseImpl(const FRigidTransform3& SpaceTransform, const FCCDParticle* CCDParticle, const FCCDConstraint& CCDConstraint, const int32 ManifoldPointIndex, const FVec3& Impulse, const FRealSingle Duration, const FChaosDebugDrawSettings& Settings)
		{
			if (CCDParticle != nullptr)
			{
				FConstGenericParticleHandle P0 = CCDParticle->Particle;
				if (P0->IsDynamic() && P0->CCDEnabled())
				{
					if (!Impulse.IsNearlyZero() && (Settings.ImpulseScale > 0))
					{
						const FVec3 Pos0 = SpaceTransform.TransformPosition(P0->PCom());
						DrawDebugLine(Pos0, Pos0 + Settings.ImpulseScale * Impulse, FColor::Red, false, Duration, uint8(Settings.DrawPriority), Settings.LineThickness * 0.5f);
					}
				}
			}
		}

		void DrawCCDCollisionImpulseImpl(const FRigidTransform3& SpaceTransform, const FCCDConstraint& CCDConstraint, const int32 ManifoldPointIndex, const FVec3& Impulse, const FChaosDebugDrawSettings& Settings)
		{
			const FRealSingle Duration = ChaosDebugDrawCCDDuration;

			DrawCollisionImpl(SpaceTransform, CCDConstraint.SweptConstraint, 1.0f, Duration, Settings);

			DrawParticleCCDCollisionImpulseImpl(SpaceTransform, CCDConstraint.Particle[0], CCDConstraint, ManifoldPointIndex, Impulse, Duration, Settings);
			DrawParticleCCDCollisionImpulseImpl(SpaceTransform, CCDConstraint.Particle[1], CCDConstraint, ManifoldPointIndex, Impulse, Duration, Settings);
		}

		void DrawCollidingShapesImpl(const FRigidTransform3& SpaceTransform, const FPBDCollisionConstraint& Collision, FRealSingle ColorScale, const FRealSingle Duration, const FChaosDebugDrawSettings& Settings)
		{
			if (!Collision.GetDisabled() && (Collision.GetPhi() < TNumericLimits<FReal>::Max()))
			{
				const FImplicitObject* Implicit0 = Collision.GetImplicit0();
				const FImplicitObject* Implicit1 = Collision.GetImplicit1();
				if ((Implicit0 != nullptr) && (Implicit1 != nullptr))
				{
					FConstGenericParticleHandle Particle0 = Collision.GetParticle0();
					FConstGenericParticleHandle Particle1 = Collision.GetParticle1();
					const FPerShapeData* Shape0 = Collision.GetShape0();
					const FPerShapeData* Shape1 = Collision.GetShape1();
					const FRigidTransform3 WorldActorTransform0 = FParticleUtilities::GetActorWorldTransform(Particle0);
					const FRigidTransform3 WorldActorTransform1 = FParticleUtilities::GetActorWorldTransform(Particle1);
					const FRigidTransform3 ShapeWorldTransform0 = Collision.GetShapeRelativeTransform0() * WorldActorTransform0;
					const FRigidTransform3 ShapeWorldTransform1 = Collision.GetShapeRelativeTransform1() * WorldActorTransform1;
					DrawShapesImpl(
						Particle0->Handle(), ShapeWorldTransform0,
						Implicit0, Shape0,
						0.0f, Particle0->IsDynamic() ? FColor::Yellow : FColor::Red,
						Duration, Settings);
					DrawShapesImpl(
						Particle1->Handle(), ShapeWorldTransform1,
						Implicit1, Shape1,
						0.0f, Particle1->IsDynamic() ? FColor::Yellow : FColor::Red,
						Duration, Settings);
				}
			}
		}
		
		void DrawCollidingShapesImpl(const FRigidTransform3& SpaceTransform, const FPBDCollisionConstraints& Collisions, FRealSingle ColorScale, const FRealSingle Duration, const FChaosDebugDrawSettings& Settings)
		{
			TArray<const FImplicitObject*> Implicits;
			TArray<const FPerShapeData*> Shapes;
			TArray<FConstGenericParticleHandle> ShapeParticles;
			TArray<FRigidTransform3> ShapeTransforms;

			for (int32 ConstraintIndex = 0; ConstraintIndex < Collisions.NumConstraints(); ++ConstraintIndex)
			{
				const FPBDCollisionConstraint* PointConstraint = &Collisions.GetConstraint(ConstraintIndex);
				if (!PointConstraint->GetDisabled() && (PointConstraint->GetPhi() < TNumericLimits<FReal>::Max()))
				{
					const FImplicitObject* Implicit0 = PointConstraint->GetImplicit0();
					const FImplicitObject* Implicit1 = PointConstraint->GetImplicit1();
					if ((Implicit0 != nullptr) && (Implicit1 != nullptr))
					{
						FConstGenericParticleHandle Particle0 = PointConstraint->GetParticle0();
						FConstGenericParticleHandle Particle1 = PointConstraint->GetParticle1();
						const FRigidTransform3 WorldActorTransform0 = FParticleUtilities::GetActorWorldTransform(Particle0);
						const FRigidTransform3 WorldActorTransform1 = FParticleUtilities::GetActorWorldTransform(Particle1);
						const FRigidTransform3 ShapeWorldTransform0 = PointConstraint->GetShapeRelativeTransform0() * WorldActorTransform0;
						const FRigidTransform3 ShapeWorldTransform1 = PointConstraint->GetShapeRelativeTransform1() * WorldActorTransform1;
						const FPerShapeData* Shape0 = PointConstraint->GetShape0();
						const FPerShapeData* Shape1 = PointConstraint->GetShape1();

						if (!Implicits.Contains(Implicit0))
						{
							Implicits.Add(Implicit0);
							Shapes.Add(Shape0);
							ShapeParticles.Add(Particle0);
							ShapeTransforms.Add(ShapeWorldTransform0);
						}

						if (!Implicits.Contains(Implicit1))
						{
							Implicits.Add(Implicit1);
							Shapes.Add(Shape1);
							ShapeParticles.Add(Particle1);
							ShapeTransforms.Add(ShapeWorldTransform1);
						}
					}
				}
			}

			for (int32 ShapeIndex = 0; ShapeIndex < Implicits.Num(); ++ShapeIndex)
			{
				DrawShapesImpl(
					ShapeParticles[ShapeIndex]->Handle(), 
					ShapeTransforms[ShapeIndex], 
					Implicits[ShapeIndex],
					Shapes[ShapeIndex],
					0.0f, 
					ShapeParticles[ShapeIndex]->IsDynamic() ? FColor::Yellow : FColor::Red, 
					Duration,
					Settings);
			}
		}

		void DrawParticleMassImpl(const FRigidTransform3& SpaceTransform, const FGeometryParticleHandle* InParticle, const FChaosDebugDrawSettings& Settings)
		{
			if (ShouldIgnoreParticle(InParticle))
			{
				return;
			}

			if (const auto RigidParticle = InParticle->CastToRigidParticle())
			{
				const FColor Color = RigidParticle->IsDynamic()? FColor::Yellow: FColor::Cyan;

				const FVec3 PCOM = SpaceTransform.TransformPosition(FParticleUtilities::GetCoMWorldPosition(RigidParticle));

				const FReal Mass = RigidParticle->M();
				if (Mass > 0 && !RigidParticle->Disabled())
				{
					auto FormatString = TEXT("{0} kg");
					FReal FormatMultiplier = 1.0;
					if (Mass < 1.0)
					{
						FormatString = TEXT("{0} g");
						FormatMultiplier = 1000.0;
					}
					else if (Mass > 1000.0)
					{
						FormatString = TEXT("{0} t");
						FormatMultiplier = 0.001;
					}
					const FReal RoundedMass = 0.1 * FMath::RoundToDouble(Mass * FormatMultiplier * 10.0);
					const FString MassStr = FString::Format(FormatString, { FString::SanitizeFloat(RoundedMass, 1) });
					DrawDebugString(PCOM, MassStr, nullptr, Color, UE_KINDA_SMALL_NUMBER, false, Settings.FontScale);
				}
			}
		}

		void DrawParticleDensityImpl(const FRigidTransform3& SpaceTransform, const FGeometryParticleHandle* InParticle, const FChaosDebugDrawSettings& Settings)
		{
			if (ShouldIgnoreParticle(InParticle))
			{
				return;
			}

			if (const auto RigidParticle = InParticle->CastToRigidParticle())
			{
				if (RigidParticle->Disabled())
				{
					return;
				}

				const FColor Color = RigidParticle->IsDynamic() ? FColor::Yellow : FColor::Cyan;

				const FVec3 PCOM = SpaceTransform.TransformPosition(FParticleUtilities::GetCoMWorldPosition(RigidParticle));

				// density is per material so it is set per shape
				for (const TUniquePtr<FPerShapeData>& Shape: RigidParticle->ShapesArray())
				{
					if (Shape && Shape->NumMaterials() > 0)
					{
						// use the first one for now
						if (const Chaos::FChaosPhysicsMaterial* ChaosMaterial = Shape->GetMaterial(0).Get())
						{
							const double Density = ChaosMaterial->Density;
							auto FormatString = TEXT("{0}");
							const FString DensityStr = FString::Format(FormatString, { FString::SanitizeFloat(Density, 1) });
							FDebugDrawQueue::GetInstance().DrawDebugString(PCOM, DensityStr, nullptr, Color, UE_KINDA_SMALL_NUMBER, false, Settings.FontScale);

						}
					}
				}
			}
		}

		void DrawJointConstraintImpl(const FRigidTransform3& SpaceTransform, const FPBDJointConstraintHandle* ConstraintHandle, Chaos::FRealSingle ColorScale, const FChaosDebugDrawJointFeatures& FeatureMask, const FChaosDebugDrawSettings& Settings)
		{
			if (!ConstraintHandle->IsEnabled())
			{
				return;
			}

			TVec2<FGeometryParticleHandle*> ConstrainedParticles = ConstraintHandle->GetConstrainedParticles();
			auto RigidParticle0 = ConstrainedParticles[0]->CastToRigidParticle();
			auto RigidParticle1 = ConstrainedParticles[1]->CastToRigidParticle();
			if ((RigidParticle0 == nullptr) && (RigidParticle1 == nullptr))
			{
				return;
			}
			if (RigidParticle0->IsDynamic() || RigidParticle0->IsSleeping() || RigidParticle1->IsDynamic() || RigidParticle1->IsSleeping())
			{
				FVec3 Pa = FParticleUtilities::GetActorWorldTransform(FConstGenericParticleHandle(ConstraintHandle->GetConstrainedParticles()[1])).GetTranslation();
				FVec3 Pb = FParticleUtilities::GetActorWorldTransform(FConstGenericParticleHandle(ConstraintHandle->GetConstrainedParticles()[0])).GetTranslation();
				FVec3 Ca = FParticleUtilities::GetCoMWorldPosition(FConstGenericParticleHandle(ConstraintHandle->GetConstrainedParticles()[1]));
				FVec3 Cb = FParticleUtilities::GetCoMWorldPosition(FConstGenericParticleHandle(ConstraintHandle->GetConstrainedParticles()[0]));
				FVec3 Xa, Xb;
				FMatrix33 Ra, Rb;
				ConstraintHandle->CalculateConstraintSpace(Xa, Ra, Xb, Rb);

				Pa = SpaceTransform.TransformPosition(Pa);
				Pb = SpaceTransform.TransformPosition(Pb);
				Ca = SpaceTransform.TransformPosition(Ca);
				Cb = SpaceTransform.TransformPosition(Cb);
				Xa = SpaceTransform.TransformPosition(Xa);
				Xb = SpaceTransform.TransformPosition(Xb);

				FColor R = FColor::Red;
				FColor G = FColor::Green;
				FColor B = FColor::Blue;
				FColor C = FColor::Cyan;
				FColor M = FColor::Magenta;
				FColor Y = FColor::Yellow;

				if (FeatureMask.bActorConnector)
				{
					const FRealSingle ConnectorThickness = 1.5f * Settings.LineThickness;
					const FReal CoMSize = Settings.DrawScale * Settings.JointComSize;
					// Leave a gap around the actor position so we can see where the center is
					FVec3 Sa = Pa;
					const FReal Lena = (Xa - Pa).Size();
					if (Lena > UE_KINDA_SMALL_NUMBER)
					{
						Sa = FMath::Lerp(Pa, Xa, FMath::Clamp<FReal>(CoMSize / Lena, 0., 1.));
					}
					FVec3 Sb = Pb;
					const FReal Lenb = (Xb - Pb).Size();
					if (Lenb > UE_KINDA_SMALL_NUMBER)
					{
						Sb = FMath::Lerp(Pb, Xb, FMath::Clamp<FReal>(CoMSize / Lena, 0., 1.));
					}
					DrawDebugLine(Pa, Sa, FColor::White, false, UE_KINDA_SMALL_NUMBER, uint8(Settings.DrawPriority), ConnectorThickness);
					DrawDebugLine(Pb, Sb, FColor::White, false, UE_KINDA_SMALL_NUMBER, uint8(Settings.DrawPriority), ConnectorThickness);
					DrawDebugLine(Sa, Xa, R, false, UE_KINDA_SMALL_NUMBER, uint8(Settings.DrawPriority), ConnectorThickness);
					DrawDebugLine(Sb, Xb, C, false, UE_KINDA_SMALL_NUMBER, uint8(Settings.DrawPriority), ConnectorThickness);
				}
				if (FeatureMask.bCoMConnector)
				{
					const FRealSingle ConnectorThickness = 1.5f * Settings.LineThickness;
					const FReal CoMSize = Settings.DrawScale * Settings.JointComSize;
					// Leave a gap around the body position so we can see where the center is
					FVec3 Sa = Ca;
					const FReal Lena = (Xa - Ca).Size();
					if (Lena > UE_KINDA_SMALL_NUMBER)
					{
						Sa = FMath::Lerp(Ca, Xa, FMath::Clamp<FReal>(CoMSize / Lena, 0., 1.));
					}
					FVec3 Sb = Cb;
					const FReal Lenb = (Xb - Cb).Size();
					if (Lenb > UE_KINDA_SMALL_NUMBER)
					{
						Sb = FMath::Lerp(Cb, Xb, FMath::Clamp<FReal>(CoMSize / Lena, 0., 1.));
					}
					DrawDebugLine(Ca, Sa, FColor::Black, false, UE_KINDA_SMALL_NUMBER, uint8(Settings.DrawPriority), ConnectorThickness);
					DrawDebugLine(Cb, Sb, FColor::Black, false, UE_KINDA_SMALL_NUMBER, uint8(Settings.DrawPriority), ConnectorThickness);
					DrawDebugLine(Sa, Xa, R, false, UE_KINDA_SMALL_NUMBER, uint8(Settings.DrawPriority), ConnectorThickness);
					DrawDebugLine(Sb, Xb, C, false, UE_KINDA_SMALL_NUMBER, uint8(Settings.DrawPriority), ConnectorThickness);
				}
				if (FeatureMask.bStretch)
				{
					const FRealSingle StretchThickness = 3.0f * Settings.LineThickness;
					DrawDebugLine(Xa, Xb, M, false, UE_KINDA_SMALL_NUMBER, uint8(Settings.DrawPriority), StretchThickness);
				}
				if (FeatureMask.bAxes)
				{
					DrawDebugDirectionalArrow(Xa, Xa + Settings.DrawScale * Settings.ConstraintAxisLen * SpaceTransform.TransformVector(Ra.GetAxis(0)), Settings.DrawScale * Settings.ArrowSize, R, false, UE_KINDA_SMALL_NUMBER, uint8(Settings.DrawPriority), Settings.LineThickness);
					DrawDebugDirectionalArrow(Xa, Xa + Settings.DrawScale * Settings.ConstraintAxisLen * SpaceTransform.TransformVector(Ra.GetAxis(1)), Settings.DrawScale * Settings.ArrowSize, G, false, UE_KINDA_SMALL_NUMBER, uint8(Settings.DrawPriority), Settings.LineThickness);
					DrawDebugDirectionalArrow(Xa, Xa + Settings.DrawScale * Settings.ConstraintAxisLen * SpaceTransform.TransformVector(Ra.GetAxis(2)), Settings.DrawScale * Settings.ArrowSize, B, false, UE_KINDA_SMALL_NUMBER, uint8(Settings.DrawPriority), Settings.LineThickness);
					DrawDebugDirectionalArrow(Xb, Xb + Settings.DrawScale * Settings.ConstraintAxisLen * SpaceTransform.TransformVector(Rb.GetAxis(0)), Settings.DrawScale * Settings.ArrowSize, C, false, UE_KINDA_SMALL_NUMBER, uint8(Settings.DrawPriority), Settings.LineThickness);
					DrawDebugDirectionalArrow(Xb, Xb + Settings.DrawScale * Settings.ConstraintAxisLen * SpaceTransform.TransformVector(Rb.GetAxis(1)), Settings.DrawScale * Settings.ArrowSize, M, false, UE_KINDA_SMALL_NUMBER, uint8(Settings.DrawPriority), Settings.LineThickness);
					DrawDebugDirectionalArrow(Xb, Xb + Settings.DrawScale * Settings.ConstraintAxisLen * SpaceTransform.TransformVector(Rb.GetAxis(2)), Settings.DrawScale * Settings.ArrowSize, Y, false, UE_KINDA_SMALL_NUMBER, uint8(Settings.DrawPriority), Settings.LineThickness);
				}

				// NOTE: GetLinearImpulse is the velocity impulse
				// (see PBDJointContainerSolver.cpp: const FVec3 LinearImpulse = -Solver.GetNetLinearImpulse() / Dt;)
				if ((Settings.PushOutScale > 0) && !ConstraintHandle->GetLinearImpulse().IsNearlyZero())
				{
					FColor PushOutImpusleColor = FColor(0, 250, 250);
					FColor Color = PushOutImpusleColor;
					DrawDebugLine(Xa, Xa + Settings.DrawScale * Settings.PushOutScale * SpaceTransform.TransformVectorNoScale(FVec3(ConstraintHandle->GetLinearImpulse())), Color, false, 0, uint8(Settings.DrawPriority), Settings.LineThickness);
				}
			}
		}

		void DrawCharacterGroundConstraintImpl(const FRigidTransform3& SpaceTransform, const FCharacterGroundConstraintHandle* Constraint, const FChaosDebugDrawSettings& Settings)
		{
			if (!Constraint->IsEnabled())
			{
				return;
			}

			const FRealSingle GroundDistance = FRealSingle(Constraint->GetData().GroundDistance);
			const FRealSingle TargetHeight = FRealSingle(Constraint->GetSettings().TargetHeight);

			if (GroundDistance < TargetHeight)
			{
				const FVec3 Pa = FParticleUtilities::GetActorWorldTransform(FConstGenericParticleHandle(Constraint->GetCharacterParticle())).GetTranslation();
				const FVec3 Start = SpaceTransform.TransformPosition(Pa);
				const FVec3 Up = SpaceTransform.TransformVector(Constraint->GetSettings().VerticalAxis);

				const FRealSingle ArrowLength = Settings.DrawScale * Settings.BodyAxisLen;
				const FRealSingle ArrowSize = Settings.DrawScale * Settings.ArrowSize;

				Chaos::FDebugDrawQueue& DrawQueue = FDebugDrawQueue::GetInstance();
				DrawQueue.DrawDebugLine(Pa, Pa - GroundDistance * Up, FColor::Emerald);
				DrawQueue.DrawDebugLine(Pa - GroundDistance * Up, Pa - TargetHeight * Up, FColor::Red);
				DrawQueue.DrawDebugDirectionalArrow(Pa - GroundDistance * Up, Pa - (GroundDistance - ArrowLength) * Up, ArrowSize, FColor::Cyan);
			}
		}

		void DrawSimulationSpaceImpl(const FTransform& SpaceTransform, const FSimulationSpace& SimSpace, const FChaosDebugDrawSettings& Settings)
		{
			const FVec3 Pos = SpaceTransform.GetLocation();
			const FRotation3& Rot = SpaceTransform.GetRotation();
			const FMatrix33 Rotm = Rot.ToMatrix();
			DrawDebugDirectionalArrow(Pos, Pos + Settings.DrawScale * Settings.BodyAxisLen * Rotm.GetAxis(0), Settings.DrawScale * Settings.ArrowSize, FColor::Red, false, UE_KINDA_SMALL_NUMBER, uint8(Settings.DrawPriority), Settings.LineThickness);
			DrawDebugDirectionalArrow(Pos, Pos + Settings.DrawScale * Settings.BodyAxisLen * Rotm.GetAxis(1), Settings.DrawScale * Settings.ArrowSize, FColor::Green, false, UE_KINDA_SMALL_NUMBER, uint8(Settings.DrawPriority), Settings.LineThickness);
			DrawDebugDirectionalArrow(Pos, Pos + Settings.DrawScale * Settings.BodyAxisLen * Rotm.GetAxis(2), Settings.DrawScale * Settings.ArrowSize, FColor::Blue, false, UE_KINDA_SMALL_NUMBER, uint8(Settings.DrawPriority), Settings.LineThickness);

			DrawDebugLine(Pos, Pos + Settings.VelScale * SpaceTransform.TransformVector(SimSpace.LinearVelocity), FColor::Cyan, false, UE_KINDA_SMALL_NUMBER, uint8(Settings.DrawPriority), Settings.LineThickness);
			DrawDebugLine(Pos, Pos + Settings.AngVelScale * SpaceTransform.TransformVector(SimSpace.AngularVelocity), FColor::Cyan, false, UE_KINDA_SMALL_NUMBER, uint8(Settings.DrawPriority), Settings.LineThickness);
			DrawDebugLine(Pos, Pos + 0.01f * Settings.VelScale * SpaceTransform.TransformVector(SimSpace.LinearAcceleration), FColor::Yellow, false, UE_KINDA_SMALL_NUMBER, uint8(Settings.DrawPriority), Settings.LineThickness);
			DrawDebugLine(Pos, Pos + 0.01f * Settings.AngVelScale * SpaceTransform.TransformVector(SimSpace.AngularAcceleration), FColor::Orange, false, UE_KINDA_SMALL_NUMBER, uint8(Settings.DrawPriority), Settings.LineThickness);
		}

		void DrawConstraintGraphImpl(const FRigidTransform3& SpaceTransform, const Private::FPBDIslandManager& Graph, const FChaosDebugDrawSettings& Settings)
		{
			auto DrawGraphCollision = [&](const FRigidTransform3& SpaceTransform, const FPBDCollisionConstraint* Constraint,  int32 IslandIndex, int32 LevelIndex, int32 ColorIndex, int32 OrderIndex, bool bIsUsed, const FChaosDebugDrawSettings& Settings)
			{
				const FRigidTransform3& ShapeTransform0 = Constraint->GetShapeWorldTransform0();
				const FRigidTransform3& ShapeTransform1 = Constraint->GetShapeWorldTransform1();
				FVec3 ContactPos = FReal(0.5) * (ShapeTransform0.GetLocation() + ShapeTransform1.GetLocation());
				if (Constraint->GetManifoldPoints().Num() > 0)
				{
					ContactPos = FVec3(0);
					for (const FManifoldPoint& ManifoldPoint : Constraint->GetManifoldPoints())
					{
						ContactPos += SpaceTransform.TransformPosition(ShapeTransform0.TransformPositionNoScale(FVec3(ManifoldPoint.ContactPoint.ShapeContactPoints[0])));
						ContactPos += SpaceTransform.TransformPosition(ShapeTransform1.TransformPositionNoScale(FVec3(ManifoldPoint.ContactPoint.ShapeContactPoints[1])));
					}
					ContactPos /= (FReal)(2 * Constraint->GetManifoldPoints().Num());
				}

				const FRigidTransform3 Transform0 = FConstGenericParticleHandle(Constraint->GetConstrainedParticles()[0])->GetTransformPQCom() * SpaceTransform;
				const FRigidTransform3 Transform1 = FConstGenericParticleHandle(Constraint->GetConstrainedParticles()[1])->GetTransformPQCom() * SpaceTransform;

				if ((bChaosDebugDebugDrawContactGraphUsed && bIsUsed) || (bChaosDebugDebugDrawContactGraphUnused && !bIsUsed))
				{
					FColor Color = bIsUsed ? FColor::Green : FColor::Red;
					DrawDebugLine(Transform0.GetLocation(), Transform1.GetLocation(), Color, false, UE_KINDA_SMALL_NUMBER, uint8(Settings.DrawPriority), Settings.LineThickness);
				}

				if (bChaosDebugDebugDrawContactGraph)
				{
					if (FConstGenericParticleHandle(Constraint->GetParticle0())->IsDynamic())
					{
						DrawDebugLine(Transform0.GetLocation(), ContactPos, FColor::Red, false, UE_KINDA_SMALL_NUMBER, uint8(Settings.DrawPriority), Settings.LineThickness);
					}
					if (FConstGenericParticleHandle(Constraint->GetParticle1())->IsDynamic())
					{
						DrawDebugLine(Transform1.GetLocation(), ContactPos, FColor::White, false, UE_KINDA_SMALL_NUMBER, uint8(Settings.DrawPriority), Settings.LineThickness);
					}
					if (ColorIndex >= 0)
					{
						DrawDebugString(ContactPos, FString::Format(TEXT("{0} L{1} C{2}"), { OrderIndex, LevelIndex, ColorIndex }), nullptr, FColor::Yellow, UE_KINDA_SMALL_NUMBER, false, Settings.FontScale);
					}
					else if (LevelIndex >= 0)
					{
						DrawDebugString(ContactPos, FString::Format(TEXT("{0} L{1}"), { OrderIndex, LevelIndex }), nullptr, FColor::Yellow, UE_KINDA_SMALL_NUMBER, false, Settings.FontScale);
					}
					else
					{
						DrawDebugString(ContactPos, FString::Format(TEXT("{0}"), { OrderIndex }), nullptr, FColor::Yellow, UE_KINDA_SMALL_NUMBER, false, Settings.FontScale);
					}
				}
			};

			if (bChaosDebugDebugDrawIslands)
			{
				for (int32 IslandIndex = 0; IslandIndex < Graph.GetNumIslands(); ++IslandIndex)
				{
					const Private::FPBDIsland* Island = Graph.GetIsland(IslandIndex);
					FAABB3 IslandAABB = FAABB3::EmptyAABB();

					for (int32 NodeIndex = 0; NodeIndex < Island->GetNumParticles(); ++NodeIndex)
					{
						FConstGenericParticleHandle Particle = Island->GetNode(NodeIndex)->GetParticle();
						IslandAABB.GrowToInclude(Particle->BoundingBox());
					}

					const FColor IslandColor = GetIslandColor(IslandIndex, !Graph.GetIsland(IslandIndex)->IsSleeping());
					const FAABB3 Bounds = IslandAABB.TransformedAABB(SpaceTransform);
					DrawDebugBox(Bounds.Center(), 0.5f * Bounds.Extents(), SpaceTransform.GetRotation(), IslandColor, false, UE_KINDA_SMALL_NUMBER, uint8(Settings.DrawPriority), 3.0f * Settings.LineThickness);
				}
			}

			if (bChaosDebugDebugDrawContactGraph || bChaosDebugDebugDrawContactGraphUnused || bChaosDebugDebugDrawContactGraphUsed)
			{
				for (int32 ContainerIndex = 0; ContainerIndex < Graph.GetNumConstraintContainers(); ++ContainerIndex)
				{
					Graph.VisitConstConstraints(ContainerIndex,
						[&Graph, &SpaceTransform, &Settings, &DrawGraphCollision](const Private::FPBDIslandConstraint* IslandConstraint)
						{
							const FConstraintHandle* Constraint = IslandConstraint->GetConstraint();
							if (const FPBDCollisionConstraintHandle* Collision = Constraint->As<FPBDCollisionConstraintHandle>())
							{
								const int32 IslandIndex = Graph.GetIslandIndex(Graph.GetConstraintIsland(Collision));
								const int32 LevelIndex = Graph.GetConstraintLevel(IslandConstraint);
								const int32 ColorIndex = Graph.GetConstraintColor(IslandConstraint);
								const int32 OrderIndex = Graph.GetIslandArrayIndex(IslandConstraint);
								const bool bIsUsed = !Collision->GetConstraint()->AccumulatedImpulse.IsNearlyZero();
								DrawGraphCollision(SpaceTransform, Collision->GetConstraint(), IslandIndex, LevelIndex, ColorIndex, OrderIndex, bIsUsed, Settings);
							}
						});
				}
			}
		}

		struct FOrderedClusterConnection {
			const FRigidClustering::FClusterHandle Handle0;
			const FRigidClustering::FClusterHandle Handle1;
			FReal Strain;

			FOrderedClusterConnection()
				: Handle0(nullptr)
				, Handle1(nullptr)
				, Strain(0)
			{}

			FOrderedClusterConnection(
				const FRigidClustering::FClusterHandle In0,
				const FRigidClustering::FClusterHandle In1,
				FReal InStrain)
				: Handle0(In0)
				, Handle1(In1)
				, Strain(InStrain)
			{
				check(In0 < In1);
			}
			bool operator==(const FOrderedClusterConnection& R) const
			{
				return Equals(R);
			}

			bool Equals(const FOrderedClusterConnection& R) const
			{
				return Handle0 == R.Handle0 && Handle1 == R.Handle1;
			}
		};

		uint32 GetTypeHash(const FOrderedClusterConnection& Object)
		{
			uint32 Hash = FCrc::MemCrc32(&Object, sizeof(FOrderedClusterConnection));
			return Hash;
		}

		void DrawConnectionGraphImpl(const FRigidClustering& Clustering, const FChaosDebugDrawSettings& Settings)
		{
			auto GetClusterTransform = [](FPBDRigidParticleHandle* Particle) -> FTransform
			{
				auto GetClusterTransformImpl = [](FPBDRigidParticleHandle* Particle, auto& Helper)->FTransform
				{
					if (!Particle)
					{
						return FTransform{};
					}

					if (FPBDRigidClusteredParticleHandle* Cluster = Particle->CastToClustered())
					{
						const ClusterId& Id = Cluster->ClusterIds();
						if (Id.Id)
						{
							return Cluster->ChildToParent() * Helper(Id.Id, Helper);
						}
					}

					return FTransform{ Particle->GetR(), Particle->GetX() };
				};

				return GetClusterTransformImpl(Particle, GetClusterTransformImpl);
			};


			auto DrawConnections = [&GetClusterTransform](TSet<FOrderedClusterConnection>& Connections, FReal MaxStrain, const FChaosDebugDrawSettings& Settings)
			{
				if (FMath::IsNearlyZero(MaxStrain))
				{
					MaxStrain = (FReal)1;
				}

				for (auto& Connection : Connections)
				{
					FVec3 Pos0 = GetClusterTransform(Connection.Handle0).GetLocation();
					FVec3 Pos1 = GetClusterTransform(Connection.Handle1).GetLocation();
					const FVec3 Normal = (Pos1 - Pos0).GetSafeNormal();
					const FReal Length = (Pos1 - Pos0).Size();
					constexpr FReal ShrinkPercentage = 0.05;
					const FVec3 SkrinkVector = Normal * ShrinkPercentage * Length;
					Pos0 += SkrinkVector;
					Pos1 -= SkrinkVector;

					FColor Color = (Connection.Handle0->Parent() == Connection.Handle1->Parent()) ? FColor::Green : FColor::Blue; // FMath::Lerp(FColor::Green, FColor::Red, Connection.Strain / MaxStrain);

					DrawDebugLine(Pos0, Pos1, Color, false, UE_KINDA_SMALL_NUMBER, uint8(Settings.DrawPriority), Settings.LineThickness);

					if (bChaosDebugDrawConnectionGraphShowAreas)
					{
						const FReal Radius = FMath::Sqrt(FMath::Abs(Connection.Strain / UE_PI)); // represent area as a circle
						const FMatrix Axes = FRotationMatrix::MakeFromX(Normal);
						DrawDebugCircle((Pos0 + Pos1) * 0.5, Radius, 24, Color, false, UE_KINDA_SMALL_NUMBER, uint8(Settings.DrawPriority), Settings.LineThickness, Axes, false);
					}


				}
			};

			auto FindChildParticlesWithLevelOffset = [&Clustering](const FRigidClustering::FClusterHandle& Cluster)
			{
				FRigidClustering::FRigidHandleArray Children;

				TQueue<TPair<FRigidClustering::FClusterHandle, int32>> Process;
				Process.Enqueue({ Cluster, 0 });

				TPair<FRigidClustering::FClusterHandle, int32> Next;
				while (Process.Dequeue(Next))
				{
					if (const FRigidClustering::FRigidHandleArray* CurrentChildren = Clustering.GetChildrenMap().Find(Next.Key))
					{
						if (Next.Value == ChaosConnectionGraphDrawLevelOffset)
						{
							Children.Append(*CurrentChildren);
						}
						else
						{
							for (FPBDRigidParticleHandle* Child : *CurrentChildren)
							{
								if (FPBDRigidClusteredParticleHandle* ClusteredChild = Child->CastToClustered())
								{
									Process.Enqueue({ClusteredChild, Next.Value + 1});
								}
							}
						}
					}
				}

				return Children;
			};

			auto VisitEveryConnectionToDraw = [&Clustering, &FindChildParticlesWithLevelOffset](auto&& Lambda)
			{
				for (const TPair<FRigidClustering::FClusterHandle, FRigidClustering::FRigidHandleArray>& ActiveCluster : Clustering.GetChildrenMap())
				{
					if (!ActiveCluster.Key->Disabled())
					{
						for (const FRigidClustering::FRigidHandle& Rigid : FindChildParticlesWithLevelOffset(ActiveCluster.Key))
						{
							if (const FRigidClustering::FClusterHandle& Child = Rigid->CastToClustered())
							{
								Lambda(Child);
							}
						}
					}
				}
			};


			FRealSingle MaxStrain = TNumericLimits<FRealSingle>::Lowest();
			int32 NumConnections = 0;
			const TArrayCollectionArray<FRealSingle>& Strain = Clustering.GetStrainArray();

			VisitEveryConnectionToDraw(
				[&MaxStrain, &NumConnections](const FRigidClustering::FClusterHandle& Child)
				{
					MaxStrain = FMath::Max(MaxStrain, Child->GetInternalStrains());
					++NumConnections;
				}
			);

			if (NumConnections)
			{
				TSet<FOrderedClusterConnection> Connections;
				Connections.Reserve(NumConnections);

				VisitEveryConnectionToDraw(
					[&Settings, &Connections, &GetClusterTransform](const FRigidClustering::FClusterHandle& Child)
					{
						FColor PointColor = Settings.ShapesColorsPerState.GetColorFromState(Child->ObjectState(), ParticleIsOneWayInteraction(Child));
						if (Child->IsAnchored())
						{
							PointColor = Settings.ShapesColorsPerState.KinematicColor;
						}
						DrawDebugPoint(GetClusterTransform(Child).GetLocation(), PointColor, false, UE_KINDA_SMALL_NUMBER, uint8(Settings.DrawPriority), Settings.LineThickness * 10);

						if (bChaosDebugDrawConnectionGraphShowInternalStrains)
						{
							DrawDebugString(Child->GetX(), FString::Printf(TEXT("%.1f"), Child->GetInternalStrains()), nullptr, FColor::White, UE_KINDA_SMALL_NUMBER, false, 1.0);
						}

						const FConnectivityEdgeArray& Edges = Child->ConnectivityEdges();
						for (const TConnectivityEdge<FReal>& Edge : Edges)
						{
							const FRigidClustering::FRigidHandle A = (Child < Edge.Sibling) ? Child : Edge.Sibling;
							const FRigidClustering::FRigidHandle B = (Child < Edge.Sibling) ? Edge.Sibling : Child;
							Connections.Add(FOrderedClusterConnection(A->CastToClustered(), B->CastToClustered(), Edge.Strain));
						}
					}
				);

				DrawConnections(Connections, MaxStrain, Settings);
			}
		}

		void DrawSuspensionConstraintsImpl(const FRigidTransform3& SpaceTransform, const FPBDSuspensionConstraints& Constraints, int32 ConstraintIndex, const FChaosDebugDrawSettings& Settings)
		{
			FConstGenericParticleHandle Particle = Constraints.GetConstrainedParticles(ConstraintIndex)[0];
			const FPBDSuspensionSettings& ConstraintSettings = Constraints.GetSettings(ConstraintIndex);
			const FPBDSuspensionResults& ConstraintResults = Constraints.GetResults(ConstraintIndex);
			
			if (!ConstraintSettings.Enabled)
			{
				return;
			}
			
			const FVec3& PLocal = Constraints.GetConstraintPosition(ConstraintIndex);
			const FRigidTransform3 ParticleTransform = Particle->GetTransformPQ();

			const FVec3 PWorld = ParticleTransform.TransformPosition(PLocal);
			const FVec3 AxisWorld = ParticleTransform.TransformVector(ConstraintSettings.Axis);
			const FReal AxisLen = ConstraintResults.Length;
			const FVec3 PushOutWorld = ConstraintResults.NetPushOut;
			const FVec3 HardStopPushOutWorld = ConstraintResults.HardStopNetPushOut;
			const FVec3 HardStopImpulseWorld = ConstraintResults.HardStopNetImpulse;

			DrawDebugLine(
				SpaceTransform.TransformPosition(PWorld), 
				SpaceTransform.TransformPosition(PWorld + AxisLen * AxisWorld), 
				FColor::Green, false, UE_KINDA_SMALL_NUMBER, uint8(Settings.DrawPriority), Settings.LineThickness);

			if (Settings.PushOutScale > 0)
			{
				// Make the shorter line wider
				const FReal SpringPushOutLen = PushOutWorld.Size();
				const FReal HardStopPushOutLen = HardStopPushOutWorld.Size();
				const FRealSingle SpringWidthScale = (SpringPushOutLen > HardStopPushOutLen) ? FRealSingle(1) : FRealSingle(2);
				const FRealSingle HardStopWidthScale = (SpringPushOutLen > HardStopPushOutLen) ? FRealSingle(2) : FRealSingle(1);

				const FColor HardStopPushOutColor = FColor(0, 200, 100);
				const FColor SpringPushOutColor = FColor(0, 200, 0);

				DrawDebugLine(
					SpaceTransform.TransformPosition(PWorld),
					SpaceTransform.TransformPosition(PWorld + Settings.PushOutScale * PushOutWorld),
					SpringPushOutColor, false, UE_KINDA_SMALL_NUMBER, uint8(Settings.DrawPriority), SpringWidthScale * Settings.LineThickness);

				DrawDebugLine(
					SpaceTransform.TransformPosition(PWorld),
					SpaceTransform.TransformPosition(PWorld + Settings.PushOutScale * HardStopPushOutWorld),
					HardStopPushOutColor, false, UE_KINDA_SMALL_NUMBER, uint8(Settings.DrawPriority), HardStopWidthScale * Settings.LineThickness);
			}

			if (Settings.ImpulseScale > 0)
			{
				const FColor HardStopImpulseColor = FColor(200, 200, 0);

				DrawDebugLine(
					SpaceTransform.TransformPosition(PWorld),
					SpaceTransform.TransformPosition(PWorld + Settings.PushOutScale * HardStopImpulseWorld),
					HardStopImpulseColor, false, UE_KINDA_SMALL_NUMBER, uint8(Settings.DrawPriority), Settings.LineThickness * FRealSingle(0.5));
			}
		}

		void DrawParticleShapes(const FRigidTransform3& SpaceTransform, const TParticleView<FGeometryParticles>& ParticlesView, FReal ColorScale, const FChaosDebugDrawSettings* Settings)
		{
			if (FDebugDrawQueue::IsDebugDrawingEnabled())
			{
				const FChaosDebugDrawSettings& DebugDrawSettings = GetChaosDebugDrawSettings(Settings);
				for (auto& Particle : ParticlesView)
				{				
					const FColor Color = GetParticleShapeDefaultColor(Particle.Handle(), DebugDrawSettings);
					DrawParticleShapesImpl(SpaceTransform, GetHandleHelper(&Particle), FConstGenericParticleHandle(Particle.Handle())->GetTransformPQ(), Color, DebugDrawSettings);
				}
			}
		}

		void DrawParticleShapes(const FRigidTransform3& SpaceTransform, const TParticleView<FKinematicGeometryParticles>& ParticlesView, FReal ColorScale, const FChaosDebugDrawSettings* Settings)
		{
			if (FDebugDrawQueue::IsDebugDrawingEnabled())
			{
				const FChaosDebugDrawSettings& DebugDrawSettings = GetChaosDebugDrawSettings(Settings);
				for (auto& Particle : ParticlesView)
				{
					const FColor Color = GetParticleShapeDefaultColor(Particle.Handle(), DebugDrawSettings);
					DrawParticleShapesImpl(SpaceTransform, GetHandleHelper(&Particle), FConstGenericParticleHandle(Particle.Handle())->GetTransformPQ(), Color, DebugDrawSettings);
				}
			}
		}

		void DrawParticleShapes(const FRigidTransform3& SpaceTransform, const TParticleView<FPBDRigidParticles>& ParticlesView, FReal ColorScale, const FChaosDebugDrawSettings* Settings)
		{
			if (FDebugDrawQueue::IsDebugDrawingEnabled())
			{
				const FChaosDebugDrawSettings& DebugDrawSettings = GetChaosDebugDrawSettings(Settings);
				for (auto& Particle : ParticlesView)
				{
					const FColor Color = GetParticleShapeDefaultColor(Particle.Handle(), DebugDrawSettings);
					DrawParticleShapesImpl(SpaceTransform, GetHandleHelper(&Particle), FConstGenericParticleHandle(Particle.Handle())->GetTransformPQ(), Color, DebugDrawSettings);
				}
			}
		}

		void DrawParticleShapes(const FRigidTransform3& SpaceTransform, const FGeometryParticleHandle* Particle, const FRigidTransform3& ParticleTransform, const FColor& Color, const FChaosDebugDrawSettings* Settings)
		{
			if (FDebugDrawQueue::IsDebugDrawingEnabled())
			{
				DrawParticleShapesImpl(SpaceTransform, Particle, ParticleTransform, Color, GetChaosDebugDrawSettings(Settings));
			}
		}

		void DrawParticleShapes(const FRigidTransform3& SpaceTransform, const FGeometryParticleHandle* Particle, const FColor& Color, const FChaosDebugDrawSettings* Settings)
		{
			if (FDebugDrawQueue::IsDebugDrawingEnabled())
			{
				DrawParticleShapesImpl(SpaceTransform, Particle, FConstGenericParticleHandle(Particle)->GetTransformPQ(), Color, GetChaosDebugDrawSettings(Settings));
			}
		}

		void DrawParticleShapes(const FRigidTransform3& SpaceTransform, const FGeometryParticle* Particle, const FColor& Color, const FChaosDebugDrawSettings* Settings)
		{
			if (FDebugDrawQueue::IsDebugDrawingEnabled())
			{
				DrawParticleShapesImpl(SpaceTransform, Particle->Handle(), FConstGenericParticleHandle(Particle->Handle())->GetTransformPQ(), Color, GetChaosDebugDrawSettings(Settings));
			}
		}

		void DrawParticleBVH(const FRigidTransform3& SpaceTransform, const FGeometryParticleHandle* Particle, const FColor& Color, const FChaosDebugDrawSettings* Settings)
		{
			if (FDebugDrawQueue::IsDebugDrawingEnabled())
			{
				DrawParticleBVHImpl(SpaceTransform, Particle, Color, GetChaosDebugDrawSettings(Settings));
			}
		}

		void DrawParticleBounds(const FRigidTransform3& SpaceTransform, const TParticleView<FGeometryParticles>& ParticlesView, const FReal Dt, const FChaosDebugDrawSettings* Settings)
		{
			if (FDebugDrawQueue::IsDebugDrawingEnabled())
			{
				for (auto& Particle : ParticlesView)
				{
					DrawParticleBoundsImpl(SpaceTransform, GetHandleHelper(&Particle), Dt, GetChaosDebugDrawSettings(Settings));
				}
			}
		}

		void DrawParticleBounds(const FRigidTransform3& SpaceTransform, const TParticleView<FKinematicGeometryParticles>& ParticlesView, const FReal Dt, const FChaosDebugDrawSettings* Settings)
		{
			if (FDebugDrawQueue::IsDebugDrawingEnabled())
			{
				for (auto& Particle : ParticlesView)
				{
					DrawParticleBoundsImpl(SpaceTransform, GetHandleHelper(&Particle), Dt, GetChaosDebugDrawSettings(Settings));
				}
			}
		}

		void DrawParticleBounds(const FRigidTransform3& SpaceTransform, const TParticleView<FPBDRigidParticles>& ParticlesView, const FReal Dt, const FChaosDebugDrawSettings* Settings)
		{
			if (FDebugDrawQueue::IsDebugDrawingEnabled())
			{
				for (auto& Particle : ParticlesView)
				{
					DrawParticleBoundsImpl(SpaceTransform, GetHandleHelper(&Particle), Dt, GetChaosDebugDrawSettings(Settings));
				}
			}
		}

		void DrawParticleTransforms(const FRigidTransform3& SpaceTransform, const TParticleView<FGeometryParticles>& ParticlesView, const FChaosDebugDrawSettings* Settings)
		{
			if (FDebugDrawQueue::IsDebugDrawingEnabled())
			{
				int32 Index = 0;
				for (auto& Particle : ParticlesView)
				{
					DrawParticleTransformImpl(SpaceTransform, GetHandleHelper(&Particle), Index++, 1.0f, GetChaosDebugDrawSettings(Settings));
				}
			}
		}

		void DrawParticleTransforms(const FRigidTransform3& SpaceTransform, const TParticleView<FKinematicGeometryParticles>& ParticlesView, const FChaosDebugDrawSettings* Settings)
		{
			if (FDebugDrawQueue::IsDebugDrawingEnabled())
			{
				int32 Index = 0;
				for (auto& Particle : ParticlesView)
				{
					DrawParticleTransformImpl(SpaceTransform, GetHandleHelper(&Particle), Index++, 1.0f, GetChaosDebugDrawSettings(Settings));
				}
			}
		}

		void DrawParticleTransforms(const FRigidTransform3& SpaceTransform, const TParticleView<FPBDRigidParticles>& ParticlesView, const FChaosDebugDrawSettings* Settings)
		{
			if (FDebugDrawQueue::IsDebugDrawingEnabled())
			{
				int32 Index = 0;
				for (auto& Particle : ParticlesView)
				{
					DrawParticleTransformImpl(SpaceTransform, GetHandleHelper(&Particle), Index++, 1.0f, GetChaosDebugDrawSettings(Settings));
				}
			}
		}

		void DrawParticleCollisions(const FRigidTransform3& SpaceTransform, const FGeometryParticleHandle* Particle, const FPBDCollisionConstraints& Collisions, const FChaosDebugDrawSettings* Settings)
		{
			if (FDebugDrawQueue::IsDebugDrawingEnabled())
			{
				for (const Chaos::FPBDCollisionConstraintHandle * ConstraintHandle : Collisions.GetConstConstraintHandles())
				{
					if (ConstraintHandle == nullptr)
					{
						continue;
					}

					TVec2<const FGeometryParticleHandle*> ConstrainedParticles = ConstraintHandle->GetConstrainedParticles();
					if ((ConstrainedParticles[0] == Particle) || (ConstrainedParticles[1] == Particle))
					{
						DrawCollisionImpl(SpaceTransform, ConstraintHandle, 1.0f, 0.0f, GetChaosDebugDrawSettings(Settings));
					}
				}
			}
		}

		void DrawCollision(const FRigidTransform3& SpaceTransform, const FPBDCollisionConstraint& Collision, FRealSingle ColorScale, const FChaosDebugDrawSettings* Settings)
		{
			if (FDebugDrawQueue::IsDebugDrawingEnabled())
			{
				DrawCollisionImpl(SpaceTransform, Collision, ColorScale, ChaosDebugDrawCollisionDuration, GetChaosDebugDrawSettings(Settings));
			}
		}

		void DrawCollisions(const FRigidTransform3& SpaceTransform, const FPBDCollisionConstraints& Collisions, FRealSingle ColorScale, const FChaosDebugDrawSettings* Settings)
		{
			if (FDebugDrawQueue::IsDebugDrawingEnabled())
			{
				for (int32 ConstraintIndex = 0; ConstraintIndex < Collisions.NumConstraints(); ++ConstraintIndex)
				{
					DrawCollisionImpl(SpaceTransform, Collisions.GetConstraint(ConstraintIndex), ColorScale, ChaosDebugDrawCollisionDuration, GetChaosDebugDrawSettings(Settings));
				}
			}
		}

		void DrawCollisions(const FRigidTransform3& SpaceTransform, const Private::FCollisionConstraintAllocator& CollisionAllocator, FRealSingle ColorScale, const FChaosDebugDrawSettings* Settings)
		{
			if (FDebugDrawQueue::IsDebugDrawingEnabled())
			{
				CollisionAllocator.VisitConstCollisions(
					[&](const FPBDCollisionConstraint& Collision)
					{
						DrawCollisionImpl(SpaceTransform, &Collision, ColorScale, ChaosDebugDrawCollisionDuration, GetChaosDebugDrawSettings(Settings));
						return ECollisionVisitorResult::Continue;
					}, ECollisionVisitorFlags::VisitAllCurrent);
			}
		}

		void DrawParticleMass(const FRigidTransform3& SpaceTransform, const TParticleView<FKinematicGeometryParticles>& ParticlesView, const FChaosDebugDrawSettings* Settings)
		{
			if (FDebugDrawQueue::IsDebugDrawingEnabled())
			{
				for (auto& Particle : ParticlesView)
				{
					DrawParticleMassImpl(SpaceTransform, GetHandleHelper(&Particle), GetChaosDebugDrawSettings(Settings));
				}
			}
		}

		void DrawParticleMass(const FRigidTransform3& SpaceTransform, const TParticleView<FPBDRigidParticles>& ParticlesView, const FChaosDebugDrawSettings* Settings)
		{
			if (FDebugDrawQueue::IsDebugDrawingEnabled())
			{
				for (auto& Particle : ParticlesView)
				{
					DrawParticleMassImpl(SpaceTransform, GetHandleHelper(&Particle), GetChaosDebugDrawSettings(Settings));
				}
			}
		}

		void DrawParticleDensity(const FRigidTransform3& SpaceTransform, const TParticleView<FKinematicGeometryParticles>& ParticlesView, const FChaosDebugDrawSettings* Settings)
		{
			if (FDebugDrawQueue::IsDebugDrawingEnabled())
			{
				for (auto& Particle : ParticlesView)
				{
					DrawParticleDensityImpl(SpaceTransform, GetHandleHelper(&Particle), GetChaosDebugDrawSettings(Settings));
				}
			}
		}

		void DrawParticleDensity(const FRigidTransform3& SpaceTransform, const TParticleView<FPBDRigidParticles>& ParticlesView, const FChaosDebugDrawSettings* Settings)
		{
			if (FDebugDrawQueue::IsDebugDrawingEnabled())
			{
				for (auto& Particle : ParticlesView)
				{
					DrawParticleDensityImpl(SpaceTransform, GetHandleHelper(&Particle), GetChaosDebugDrawSettings(Settings));
				}
			}
		}

		void DrawJointConstraints(const FRigidTransform3& SpaceTransform, const TArray<FPBDJointConstraintHandle*>& ConstraintHandles, Chaos::FRealSingle ColorScale, const FChaosDebugDrawJointFeatures& FeatureMask, const FChaosDebugDrawSettings* Settings)
		{
			if (FDebugDrawQueue::IsDebugDrawingEnabled())
			{
				for (const FPBDJointConstraintHandle* ConstraintHandle : ConstraintHandles)
				{
					DrawJointConstraintImpl(SpaceTransform, ConstraintHandle, ColorScale, FeatureMask, GetChaosDebugDrawSettings(Settings));
				}
			}
		}

		void DrawJointConstraints(const FRigidTransform3& SpaceTransform, const FPBDJointConstraints& Constraints, Chaos::FRealSingle ColorScale, const FChaosDebugDrawJointFeatures& FeatureMask, const FChaosDebugDrawSettings* Settings)
		{
			if (FDebugDrawQueue::IsDebugDrawingEnabled())
			{
				for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.NumConstraints(); ++ConstraintIndex)
				{
					DrawJointConstraintImpl(SpaceTransform, Constraints.GetConstraintHandle(ConstraintIndex), ColorScale, FeatureMask, GetChaosDebugDrawSettings(Settings));
				}
			}
		}

		void DrawSimulationSpace(const FTransform& SpaceTransform, const FSimulationSpace& SimSpace, const FChaosDebugDrawSettings* Settings)
		{
			if (FDebugDrawQueue::IsDebugDrawingEnabled())
			{
				DrawSimulationSpaceImpl(SpaceTransform, SimSpace, GetChaosDebugDrawSettings(Settings));
			}
		}

		void DrawConstraintGraph(const FRigidTransform3& SpaceTransform, const Private::FPBDIslandManager& Graph, const FChaosDebugDrawSettings* Settings)
		{
			if (FDebugDrawQueue::IsDebugDrawingEnabled())
			{
				DrawConstraintGraphImpl(SpaceTransform, Graph, GetChaosDebugDrawSettings(Settings));
			}
		}

		void DrawConnectionGraph(const FRigidClustering& Clustering, const FChaosDebugDrawSettings* Settings)
		{
			if (FDebugDrawQueue::IsDebugDrawingEnabled())
			{
				DrawConnectionGraphImpl(Clustering, GetChaosDebugDrawSettings(Settings));
			}
		}

		void DrawCollidingShapes(const FRigidTransform3& SpaceTransform, const FPBDCollisionConstraints& Collisions, FRealSingle ColorScale, const FRealSingle Duration, const FChaosDebugDrawSettings* Settings)
		{
			if (FDebugDrawQueue::IsDebugDrawingEnabled())
			{
				DrawCollidingShapesImpl(SpaceTransform, Collisions, ColorScale, Duration, GetChaosDebugDrawSettings(Settings));
			}
		}

		void DrawCollidingShapes(const FRigidTransform3& SpaceTransform, const FPBDCollisionConstraint& Collision, FRealSingle ColorScale, const FRealSingle Duration, const FChaosDebugDrawSettings* Settings)
		{
			if (FDebugDrawQueue::IsDebugDrawingEnabled())
			{
				DrawCollidingShapesImpl(SpaceTransform, Collision, ColorScale, Duration, GetChaosDebugDrawSettings(Settings));
			}
		}

		class FSpatialDebugDrawInterface: public ISpatialDebugDrawInterface
		{
		public:
			FSpatialDebugDrawInterface(const FChaosDebugDrawSettings& InSettings)
				: Settings(InSettings)
			{}
			
			virtual ~FSpatialDebugDrawInterface() override = default;

			virtual void Box(const FAABB3& InBox, const TVec3<FReal>& InLinearColor, float InThickness) override
			{
				DrawDebugBox(InBox.Center(), InBox.Extents() * FReal(0.5), FQuat::Identity, FLinearColor(InLinearColor).ToFColor(false), false, -1.f, uint8(Settings.DrawPriority), InThickness);
			}

			virtual void Line(const TVec3<FReal>& InBegin, const TVec3<FReal>& InEnd, const TVec3<FReal>& InLinearColor, float InThickness) override
			{
				DrawDebugLine(InBegin, InEnd, FLinearColor(InLinearColor).ToFColor(false), false, -1.f, uint8(Settings.DrawPriority), InThickness);
			}
		private:
			FChaosDebugDrawSettings Settings;
		};

		void DrawSpatialAccelerationStructure(const ISpatialAcceleration<FAccelerationStructureHandle, FReal, 3>& InSpatialAccelerationStructure, const FChaosDebugDrawSettings* InSettings)
		{
		#if !UE_BUILD_SHIPPING
			FSpatialDebugDrawInterface DebugDrawInterface(GetChaosDebugDrawSettings(InSettings));
			InSpatialAccelerationStructure.DebugDraw(&DebugDrawInterface);
		#endif
		}

		void DrawSuspensionConstraints(const FRigidTransform3& SpaceTransform, const FPBDSuspensionConstraints& Constraints, const FChaosDebugDrawSettings* Settings)
		{
			if (FDebugDrawQueue::IsDebugDrawingEnabled())
			{
				for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.NumConstraints(); ++ConstraintIndex)
				{
					DrawSuspensionConstraintsImpl(SpaceTransform, Constraints, ConstraintIndex, GetChaosDebugDrawSettings(Settings));
				}
			}
		}

		void DrawCharacterGroundConstraints(const FRigidTransform3& SpaceTransform, const FCharacterGroundConstraintContainer& Constraints, const FChaosDebugDrawSettings* Settings)
		{
			if (FDebugDrawQueue::IsDebugDrawingEnabled())
			{
				for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.NumConstraints(); ++ConstraintIndex)
				{
					DrawCharacterGroundConstraintImpl(SpaceTransform, Constraints.GetConstraint(ConstraintIndex), GetChaosDebugDrawSettings(Settings));
				}
			}
		}

		void DrawCCDCollisionImpulse(const FRigidTransform3& SpaceTransform, const FCCDConstraint& CCDConstraint, const int32 ManifoldPointIndex, const FVec3& Impulse, const FChaosDebugDrawSettings* Settings)
		{
			if (FDebugDrawQueue::IsDebugDrawingEnabled())
			{
				DrawCCDCollisionImpulseImpl(SpaceTransform, CCDConstraint, ManifoldPointIndex, Impulse, GetChaosDebugDrawSettings(Settings));
			}
		}

		void DrawCCDAxisThreshold(const FVec3& X, const FVec3& AxisThreshold, const FVec3& DeltaX, const FQuat& R)
		{
			// Call the version of DeltaExceedsThreshold that provides access to its
			// intermediate variables so we can debug draw them
			FVec3 AbsLocalDelta, AxisThresholdScaled, AxisThresholdDiff;
			const bool bEnableCCD = FCCDHelpers::DeltaExceedsThreshold(
				AxisThreshold, DeltaX, R, AbsLocalDelta, AxisThresholdScaled, AxisThresholdDiff);
			const FVector AxisPercents = AbsLocalDelta / AxisThresholdScaled;

			// Pick draw settings
			const FColor BoundsColor = bEnableCCD ? FColor::Orange : FColor::Cyan;
			const FColor DeltaColor = FColor::Red;
			const float ArrowSize = 20.f;
			const FString AxisStrings[] = { FString("X"), FString("Y"), FString("Z") };
			const float AxisThickness = 4.f;
			const float DeltaThickness = 2.f;
			const float BoundsThickness = 2.f;

			const auto DrawAxis = [&](const int32 AxisIndex)
			{
				const bool bAxisEnableCCD = AxisThresholdDiff[AxisIndex] > 0.f;
				const FColor AxisColor = bEnableCCD ? FColor::Orange : FColor::Cyan;
				FVector AxisLocalExtent = FVector::ZeroVector;
				FVector AxisLocalDelta = FVector::ZeroVector;
				AxisLocalExtent[AxisIndex] = AxisThresholdScaled[AxisIndex];
				AxisLocalDelta[AxisIndex] = AbsLocalDelta[AxisIndex];
				const FVector AxisExtent = R.RotateVector(AxisLocalExtent);
				const FVector AxisDelta = R.RotateVector(AxisLocalDelta);
				DrawDebugDirectionalArrow(X, X + AxisExtent, ArrowSize, AxisColor, false, -1.f, -1, AxisThickness);
				DrawDebugDirectionalArrow(X, X + AxisDelta, ArrowSize, DeltaColor, false, -1.f, -1, DeltaThickness);
				DrawDebugString(X + AxisExtent, FString::Printf(TEXT("%s: %.2f%%"), *AxisStrings[AxisIndex], AxisPercents[AxisIndex] * 100.f), nullptr, AxisColor, -1.f, true, 1.f);
			};

			// Get the index of the largest element
			const auto MaxIndex = [](const FVector& Vec)
			{
				return Vec[0] > Vec[1]
					? (Vec[0] > Vec[2] ? (uint8)0 : (uint8)2)
					: (Vec[1] > Vec[2] ? (uint8)1 : (uint8)2);
			};

			// Draw the axis which has the highest percentage of it's available position delta
			const int32 AxisIndexMaxPercent = MaxIndex(AxisPercents);
			DrawAxis(AxisIndexMaxPercent);

			// Draw a box representing the size of the CCD extents on each axis at the start
			// and end position of the CoM over this frame (gray = no ccd, orange = ccd enabled)
			// Also draw a position delta vector (red) and the threshold delta vector (green)
			const FVector HalfExtents = AxisThreshold * .5f;
			DrawDebugBox(X, HalfExtents, R, BoundsColor, false, -1.f, -1, BoundsThickness);
			DrawDebugBox(X + DeltaX, HalfExtents, R, BoundsColor, false, -1.f, -1, BoundsThickness);
			const float ScaleArray[] = { -1.f, 1.f };
			for (uint8 IX = 0; IX < 2; ++IX)
			{
				for (uint8 IY = 0; IY < 2; ++IY)
				{
					for (uint8 IZ = 0; IZ < 2; ++IZ)
					{
						const FVector LocalOffset = HalfExtents * FVector(ScaleArray[IX], ScaleArray[IY], ScaleArray[IZ]);
						const FVector Offset = R.RotateVector(LocalOffset);
						DrawDebugLine(X + Offset, X + Offset + DeltaX, BoundsColor, false, -1.f, -1, BoundsThickness);
					}
				}
			}
		}

	}	// namespace DebugDraw
}	// namespace Chaos

#endif
