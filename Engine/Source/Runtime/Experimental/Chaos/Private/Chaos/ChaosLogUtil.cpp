// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/ChaosLogUtil.h"
#include "CoreTypes.h"

namespace Chaos
{
	FString ToString(const EObjectStateType ObjectState)
	{
		static FString StateStrings[] =
		{
			TEXT("Uninitialized(0)"),
			TEXT("Sleeping(1)"),
			TEXT("Kinematic(2)"),
			TEXT("Static(3)"),
			TEXT("Dynamic(4)"),
		};

		const bool bIsValid = ((int32)ObjectState >= 0) && ((int32)ObjectState < (int32)EObjectStateType::Count);
		if (ensure(bIsValid))
		{
			return StateStrings[(int32)ObjectState];
		}

		return FString::Printf(TEXT("Invalid(%d)"), (int32)ObjectState);
	}
}