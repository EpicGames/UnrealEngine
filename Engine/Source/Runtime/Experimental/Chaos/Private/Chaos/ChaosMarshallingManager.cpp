// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/ChaosMarshallingManager.h"
#include "Chaos/PullPhysicsDataImp.h"
#include "Chaos/AsyncInitBodyHelper.h"

namespace Chaos
{

int32 SimDelay = 0;
FAutoConsoleVariableRef CVarSimDelay(TEXT("p.simDelay"),SimDelay,TEXT(""));

bool bCachePushDataDirtyProxies = true;
FAutoConsoleVariableRef CVarCachePushDataDirtyProxies(TEXT("p.Resim.CachePushDataDirtyProxies"), bCachePushDataDirtyProxies, TEXT("Default = false. Set true to enable resim caching dirty proxies in the push data from game thread to physics thread. This will make physics proxy changes from GT play out during resimulation."));

bool bCachePushDataAsyncInputs = true;
FAutoConsoleVariableRef CVarCachePushDataAsyncInputs(TEXT("p.Resim.CachePushDataAsyncInputs"), bCachePushDataAsyncInputs, TEXT("Default = false. Set true to enable resim caching of async inputs in the push data from game thread to physics thread. This will make async inputs available again during resimulation."));

bool bCachePushDataSimCommands = true;
FAutoConsoleVariableRef CVarCachePushDataSimCommands(TEXT("p.Resim.CachePushDataSimCommands"), bCachePushDataSimCommands, TEXT("Default = false. Set true to enable resim caching of sim commands in the push data from game thread to physics thread. This will make sim commands available again during resimulation."));

FChaosMarshallingManager::FChaosMarshallingManager()
: ExternalTime_External(0)
, ExternalTimestamp_External(0)
, SimTime_External(0)
, InternalStep_External(0)
, ConsumerData(nullptr)
, ProducerData(nullptr)
, CurPullData(nullptr)
, Delay(SimDelay)
, HistoryLength(0)
{
	PrepareExternalQueue_External();
	PreparePullData();
}

FChaosMarshallingManager::~FChaosMarshallingManager()
{
	SetHistoryLength_Internal(0);	//ensure anything in pending history is cleared
}

void FChaosMarshallingManager::FinalizePullData_Internal(int32 LastExternalTimestampConsumed, FReal SimStartTime, FReal DeltaTime)
{
	CurPullData->SolverTimestamp = LastExternalTimestampConsumed;
	CurPullData->ExternalStartTime = SimStartTime;
	CurPullData->ExternalEndTime = SimStartTime + DeltaTime;
	PullDataQueue.Enqueue(CurPullData);
	PreparePullData();
}

void FChaosMarshallingManager::PreparePullData()
{
	if(!PullDataPool.Dequeue(CurPullData))
	{
		const int32 Idx = BackingPullBuffer.Add(MakeUnique<FPullPhysicsData>());
		CurPullData = BackingPullBuffer[Idx].Get();
	}
}

void FChaosMarshallingManager::PrepareExternalQueue_External()
{
	// Here, we assume that MarshallingManagerLock is locked when p.Chaos.EnableAsyncInitBody is true (see FPBDRigidsSolver::PushPhysicsState)
	if(!PushDataPool.Dequeue(ProducerData))
	{
		BackingBuffer.Add(MakeUnique<FPushPhysicsData>());
		ProducerData = BackingBuffer.Last().Get();
	}

	ProducerData->StartTime = ExternalTime_External;
}

void FChaosMarshallingManager::Step_External(FReal ExternalDT, const int32 NumSteps, bool bInSolverSubstepped)
{
	// Here, we assume that MarshallingManagerLock is locked when p.Chaos.EnableAsyncInitBody is true (see FPBDRigidsSolver::PushPhysicsState)
	ensure(NumSteps > 0);

	FPushPhysicsData* FirstStepData = nullptr;
	for(int32 Step = 0; Step < NumSteps; ++Step)
	{
		for (int32 Idx = ProducerData->SimCallbackInputs.Num() -1; Idx >= 0; --Idx)
		{
			FSimCallbackInputAndObject& Pair = ProducerData->SimCallbackInputs[Idx];
			Pair.CallbackObject->CurrentExternalInput_External = nullptr;	//mark data as marshalled, any new data must be in a new data packet
			Pair.Input->SetNumSteps_External(NumSteps);

			if(Pair.CallbackObject->bPendingDelete_External)
			{
				ProducerData->SimCallbackInputs.RemoveAtSwap(Idx);
			}
		}

		//stored in reverse order for easy removal later. Might want to use a circular buffer if perf is bad here
		//expecting queue to be fairly small (3,4 at most) so probably doesn't matter
		ProducerData->ExternalDt = ExternalDT;
		ProducerData->ExternalTimestamp = ExternalTimestamp_External;
		ProducerData->InternalStep = InternalStep_External++;
		ProducerData->IntervalStep = Step;
		ProducerData->IntervalNumSteps = NumSteps;
		ProducerData->bSolverSubstepped = bInSolverSubstepped;

		ExternalQueue.Insert(ProducerData, 0);

		if(Step == 0)
		{
			FirstStepData = ProducerData;
		}
		else
		{
			//copy sub-step only data
			ProducerData->CopySubstepData(*FirstStepData);
		}

		ExternalTime_External.store(ExternalTime_External.load() + ExternalDT);
		PrepareExternalQueue_External();
	}

	++ExternalTimestamp_External;
}

FPushPhysicsData* FChaosMarshallingManager::StepInternalTime_External()
{
	UE_CHAOS_ASYNC_INITBODY_WRITESCOPELOCK(MarshallingManagerLock);
	if (Delay == 0)
	{
		if(ExternalQueue.Num())
		{
			return ExternalQueue.Pop(EAllowShrinking::No);
		}
	}
	else
	{
		--Delay;
	}

	return nullptr;
}

void FChaosMarshallingManager::FreeData_Internal(FPushPhysicsData* PushData)
{
	UE_CHAOS_ASYNC_INITBODY_WRITESCOPELOCK(MarshallingManagerLock);
	if (PushData == ConsumerData)
	{
		ConsumerData = nullptr;
	}
	PushData->ResetDirtyProxiesBuffer();
	PushData->Reset();
	PushDataPool.Enqueue(PushData);
}

void FChaosMarshallingManager::FreePullData_External(FPullPhysicsData* PullData)
{
	PullData->Reset();
	PullDataPool.Enqueue(PullData);
}

void FPushPhysicsData::Reset()
{
	ClearAsyncInputs();
	
	for(ISimCallbackObject* CallbackToRemove : SimCallbackObjectsToRemove)
	{
		ensure(CallbackToRemove->bPendingDelete);	//should already be marked pending delete
		delete CallbackToRemove;
	}

	ClearSimCommands();
	DirtyProxiesDataBuffer.Reset();
	SimCallbackObjectsToAdd.Reset();
	SimCallbackObjectsToRemove.Reset();
}

void FPushPhysicsData::ResetForHistory()
{
	// Clear dirty proxies data if we should not cache them in history
	if (bCachePushDataDirtyProxies == false)
	{
		ResetDirtyProxiesBuffer();
		DirtyProxiesDataBuffer.Reset();
	}

	// Clear async inputs if we should not cache them in history
	if (bCachePushDataAsyncInputs == false)
	{
		ClearAsyncInputs();
	}

	// Clear simcommands if we should not cache them in history
	if (bCachePushDataSimCommands == false)
	{
		ClearSimCommands();
	}

	SimCallbackObjectsToAdd.Reset();
}

void FPushPhysicsData::ResetDirtyProxiesBuffer()
{
	//TODO: we know entire manager is cleared, so we can probably just iterate over its pools and reset
	//instead of going through dirty proxies. If perf matters fix this
	FDirtyPropertiesManager* Manager = &DirtyPropertiesManager;
	FShapeDirtyData* ShapeDirtyData = DirtyProxiesDataBuffer.GetShapesDirtyData();

	DirtyProxiesDataBuffer.ForEachProxy([Manager, ShapeDirtyData](int32 DataIdx, FDirtyProxy& Dirty)
		{
			Dirty.Clear(*Manager, DataIdx, ShapeDirtyData);
		});
}

void FPushPhysicsData::ClearAsyncInputs()
{
	for (FSimCallbackInputAndObject& CallbackInputAndObject : SimCallbackInputs)
	{
		if (CallbackInputAndObject.CallbackObject)
		{
			CallbackInputAndObject.Input->Release_Internal(*CallbackInputAndObject.CallbackObject);
		}
	}
	SimCallbackInputs.Reset();
}

void FPushPhysicsData::ClearSimCommands()
{
	for (FSimCallbackCommandObject* SimCallbackObject : SimCommands)
	{
		if (SimCallbackObject)
		{
			delete SimCallbackObject;
		}
	}
	SimCommands.Reset();
}

void FPushPhysicsData::ClearAsyncInputForCallback(ISimCallbackObject* Callback)
{
	for (TArray<FSimCallbackInputAndObject>::TIterator Itr = SimCallbackInputs.CreateIterator(); Itr; ++Itr)
	{
		FSimCallbackInputAndObject& CallbackInputAndObject = *Itr;
		if (CallbackInputAndObject.CallbackObject == Callback)
		{
			CallbackInputAndObject.Input->Release_Internal(*CallbackInputAndObject.CallbackObject);
			Itr.RemoveCurrent();
		}
	}
}

void FChaosMarshallingManager::FreeDataToHistory_Internal(FPushPhysicsData* PushData)
{
	if (PushData == ConsumerData)
	{
		ConsumerData = nullptr;
	}

	if (HistoryLength == 0)
	{
		FreeData_Internal(PushData);
	}
	else
	{
		/** Process SimCallbackObjects that are pending delete and clear their cached AsyncInputs from history
		* At this point the SimCallbackObject is already unregistered from all callbacks and keeping it alive longer (until the end of the rewind history) is only a risk */
		for (ISimCallbackObject* CallbackToRemove : PushData->SimCallbackObjectsToRemove)
		{
			ensure(CallbackToRemove->bPendingDelete);	//should already be marked pending delete

			// Iterate over cached history and remove any async input that is linked to the SimCallbackObject that is getting deleted
			for (FPushPhysicsData* PushDataHistoryEntry : HistoryQueue_Internal)
			{
				PushDataHistoryEntry->ClearAsyncInputForCallback(CallbackToRemove);
			}
			delete CallbackToRemove;
		}
		PushData->SimCallbackObjectsToRemove.Reset();

		PushData->ResetForHistory();
		HistoryQueue_Internal.Insert(PushData, 0);
		SetHistoryLength_Internal(HistoryLength);
	}
}

void FChaosMarshallingManager::SetHistoryLength_Internal(int32 InHistoryLength)
{
	if (!ensure(InHistoryLength >= 0))
	{
		InHistoryLength = 0;
	}
	
	HistoryLength = InHistoryLength;
	//make sure late entries are pruned
	if(HistoryQueue_Internal.Num() > HistoryLength)
	{
		//need to go from oldest to latest (back to front) since we may delete callback at latest, but still have inputs for it to free from earlier frames
		for(int32 Idx = HistoryQueue_Internal.Num() - 1; Idx >= HistoryLength; --Idx)
		{
			FreeData_Internal(HistoryQueue_Internal[Idx]);
		}
		HistoryQueue_Internal.SetNum(InHistoryLength, EAllowShrinking::No);
	}
}

void FChaosMarshallingManager::DebugHistory_Internal(int32 NumFrames)
{
	const int32 UseNumFrames = FMath::Min(NumFrames, HistoryQueue_Internal.Num());
	for (int32 Idx = 0; Idx < UseNumFrames; ++Idx)
	{
		UE_LOGF(LogChaos, Log, "					History internal[%d] at frame %d", Idx, HistoryQueue_Internal[Idx]->InternalStep);
	}
}

TArray<FPushPhysicsData*> FChaosMarshallingManager::StealHistory_Internal(int32 NumFrames)
{
	ensure(NumFrames <= HistoryQueue_Internal.Num());
	const int32 UseNumFrames = FMath::Min(NumFrames, HistoryQueue_Internal.Num());

	TArray<FPushPhysicsData*> History;
	History.Reserve(UseNumFrames);

	for(int32 Idx = 0; Idx < UseNumFrames; ++Idx)
	{
		History.Add(HistoryQueue_Internal[Idx]);
	}
	HistoryQueue_Internal.RemoveAt(0, UseNumFrames);
	return History;
}

void FPushPhysicsData::CopySubstepData(const FPushPhysicsData& FirstStepData)
{
	const FDirtyPropertiesManager& FirstManager = FirstStepData.DirtyPropertiesManager;
	DirtyPropertiesManager.PrepareBuckets(FirstStepData.DirtyProxiesDataBuffer.GetDirtyProxyBucketInfo());
	FirstStepData.DirtyProxiesDataBuffer.ForEachProxy([this, &FirstManager](int32 FirstDataIdx, const FDirtyProxy& Dirty)
	{
		//todo: use bucket type directly instead of iterating over each proxy
		if (Dirty.Proxy->GetType() == EPhysicsProxyType::SingleParticleProxy && Dirty.PropertyData.GetParticleBufferType() == EParticleType::Rigid)
		{
			if (const FParticleDynamics* DynamicsData = Dirty.PropertyData.FindDynamics(FirstManager, FirstDataIdx))
			{
				if (DynamicsData->Acceleration() != FVec3(0) || DynamicsData->AngularAcceleration() != FVec3(0))	//don't bother interpolating 0. This is important because the input dirtys rewind data
				{
					DirtyProxiesDataBuffer.Add(Dirty.Proxy);
					FParticleDynamics& SubsteppedDynamics = DirtyPropertiesManager.GetChaosPropertyPool<FParticleDynamics, EChaosProperty::Dynamics>().GetElement(Dirty.Proxy->GetDirtyIdx());
					SubsteppedDynamics = *DynamicsData;
					//we don't want to sub-step impulses so those are cleared in the sub-step
					SubsteppedDynamics.SetAngularImpulseVelocity(FVec3(0));
					SubsteppedDynamics.SetLinearImpulseVelocity(FVec3(0));
					FDirtyProxy& NewDirtyProxy = DirtyProxiesDataBuffer.GetDirtyProxyAt(Dirty.Proxy->GetType(), Dirty.Proxy->GetDirtyIdx());
					NewDirtyProxy.PropertyData.DirtyFlag(EChaosPropertyFlags::Dynamics);
					NewDirtyProxy.PropertyData.SetParticleBufferType(EParticleType::Rigid);
				}
			}

			Dirty.Proxy->ResetDirtyIdx();	//dirty idx is only used temporarily
		}
		else if (Dirty.Proxy->GetType() == EPhysicsProxyType::ClusterUnionProxy)
		{
			DirtyProxiesDataBuffer.Add(Dirty.Proxy);
			Dirty.Proxy->ResetDirtyIdx();
		}
	});

	//make sure inputs are available to every sub-step
	SimCallbackInputs = FirstStepData.SimCallbackInputs;
}

FSimCallbackInput* ISimCallbackObject::GetProducerInputData_External()
{
	if (CurrentExternalInput_External == nullptr)
	{
		FChaosMarshallingManager& Manager = Solver->GetMarshallingManager();
		CurrentExternalInput_External = AllocateInputData_External();
		Manager.AddSimCallbackInputData_External(this, CurrentExternalInput_External);
	}

	return CurrentExternalInput_External;
}

}
