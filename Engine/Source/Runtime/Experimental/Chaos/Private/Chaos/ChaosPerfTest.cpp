// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/ChaosPerfTest.h"

#if CHAOS_PERF_TEST_ENABLED
const TCHAR* FChaosScopedDurationTimeLogger::GlobalLabel = nullptr;
EChaosPerfUnits FChaosScopedDurationTimeLogger::GlobalUnits = EChaosPerfUnits::S;
#endif
