// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/Character/CharacterGroundConstraint.h"

namespace Chaos
{
	FCharacterGroundConstraint::FCharacterGroundConstraint()
		: FConstraintBase(EConstraintType::CharacterGroundConstraintType)
		, SolverAppliedForce(FVector(0))
		, SolverAppliedTorque(FVector(0))
	{
	}

	void FCharacterGroundConstraint::Init(FSingleParticlePhysicsProxy* InCharacterProxy)
	{
		ensure(InCharacterProxy);

		CharacterProxy.Modify(/*bInvalidate=*/true, DirtyFlags, Proxy, [InCharacterProxy](FParticleProxyProperty& Data)
			{
				Data.ParticleProxy = InCharacterProxy;
			});

		// Mark the settings and data as dirty as this is required to be pushed to the
		// physics thread to initialize the data there and there is a chance the user
		// does not set any settings or data before that happens
		DirtyFlags.MarkDirty(EChaosPropertyFlags::CharacterGroundConstraintSettings);
		DirtyFlags.MarkDirty(EChaosPropertyFlags::CharacterGroundConstraintDynamicData);
	}

	void FCharacterGroundConstraint::SyncRemoteDataImp(FDirtyPropertiesManager& Manager, int32 DataIdx, FDirtyChaosProperties& RemoteData)
	{
		CharacterProxy.SyncRemote(Manager, DataIdx, RemoteData);
		GroundProxy.SyncRemote(Manager, DataIdx, RemoteData);
		ConstraintSettings.SyncRemote(Manager, DataIdx, RemoteData);
		ConstraintData.SyncRemote(Manager, DataIdx, RemoteData);
	}
}