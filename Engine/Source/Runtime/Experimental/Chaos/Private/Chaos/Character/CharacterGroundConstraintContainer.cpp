// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/Character/CharacterGroundConstraintContainer.h"

#include "Chaos/Island/IslandManager.h"
#include "CharacterGroundConstraintContainerSolver.h"

namespace Chaos
{
	namespace CVars
	{
		float Chaos_CharacterGroundConstraint_InputMovementThreshold = 0.1f;
		FAutoConsoleVariableRef CVarChaos_CharacterGroundConstraint_InputMovementThreshold(TEXT("p.Chaos.CharacterGroundConstraint.InputMovementThreshold"), Chaos_CharacterGroundConstraint_InputMovementThreshold, TEXT("Minimum per frame input movement distance in cm."));
		
		float Chaos_CharacterGroundConstraint_ExternalMovementThreshold = 2.0f;
		FAutoConsoleVariableRef CVarChaos_CharacterGroundConstraint_ExternalMovementThreshold(TEXT("p.Chaos.CharacterGroundConstraint.ExternalMovementThreshold"), Chaos_CharacterGroundConstraint_ExternalMovementThreshold, TEXT("If distance moved is less than this then retain current movement target relative to ground."));
	}
	
	FCharacterGroundConstraintContainer::FCharacterGroundConstraintContainer()
		: Base(FCharacterGroundConstraintHandle::StaticType())
		, ConstraintPool(16, 0)
	{
	}

	FCharacterGroundConstraintContainer::~FCharacterGroundConstraintContainer()
	{
	}

	void FCharacterGroundConstraintContainer::SetConstraintEnabled(int32 ConstraintIndex, bool bEnabled)
	{
		if (ConstraintIndex < Constraints.Num())
		{
			FCharacterGroundConstraintHandle* Constraint = Constraints[ConstraintIndex];
			const FGenericParticleHandle CharacterParticle = FGenericParticleHandle(Constraint->GetCharacterParticle());
			if (bEnabled)
			{
				if (CharacterParticle->Handle() != nullptr && !CharacterParticle->Disabled())
				{
					Constraint->SetEnabled(true);
				}
			}
			else
			{
				// desirable to allow disabling no matter what state the endpoints
				Constraint->SetEnabled(false);
			}
		}
	}

	FCharacterGroundConstraintHandle* FCharacterGroundConstraintContainer::AddConstraint(
		const FCharacterGroundConstraintSettings& InConstraintSettings,
		const FCharacterGroundConstraintDynamicData& InConstraintData,
		FGeometryParticleHandle* CharacterParticle,
		FGeometryParticleHandle* GroundParticle)
	{
		FCharacterGroundConstraintHandle* Constraint = ConstraintPool.Alloc();
		Constraints.Add(Constraint);

		check(CharacterParticle);
		CharacterParticle->AddConstraintHandle(Constraint);
		if (GroundParticle != nullptr)
		{
			GroundParticle->AddConstraintHandle(Constraint);
		}

		Constraint->CharacterParticle = CharacterParticle;
		Constraint->GroundParticle = GroundParticle;
		Constraint->Settings = InConstraintSettings;
		Constraint->Data = InConstraintData;
		Constraint->SetContainer(this);

		// If our character particle is disabled, so is the constraint for now.
		const bool bStartDisabled = FConstGenericParticleHandle(CharacterParticle)->Disabled();
		Constraint->bDisabled = bStartDisabled;

		return Constraint;
	}

	void FCharacterGroundConstraintContainer::RemoveConstraint(FCharacterGroundConstraintHandle* Constraint)
	{
		if (Constraint != nullptr)
		{
			if (Constraint->CharacterParticle != nullptr)
			{
				Constraint->CharacterParticle->RemoveConstraintHandle(Constraint);
			}
			if (Constraint->GroundParticle != nullptr)
			{
				Constraint->GroundParticle->RemoveConstraintHandle(Constraint);
			}

			ConstraintPool.Free(Constraint);
			Constraints.Remove(Constraint);
		}
	}

	TUniquePtr<FConstraintContainerSolver> FCharacterGroundConstraintContainer::CreateSceneSolver(const int32 Priority)
	{
		return MakeUnique<Private::FCharacterGroundConstraintContainerSolver>(*this, Priority);
	}

	TUniquePtr<FConstraintContainerSolver> FCharacterGroundConstraintContainer::CreateGroupSolver(const int32 Priority)
	{
		return MakeUnique<Private::FCharacterGroundConstraintContainerSolver>(*this, Priority);
	}

	void FCharacterGroundConstraintContainer::AddConstraintsToGraph(Private::FPBDIslandManager& IslandManager)
	{
		for (FCharacterGroundConstraintHandle* Constraint : GetConstraints())
		{
			const bool bIsInGraph = Constraint->IsInConstraintGraph();
			const bool bShouldBeInGraph = Constraint->IsEnabled() && CanEvaluate(Constraint);
			const bool bShouldUpdateGraph = Constraint->GetGroundParticle() && Constraint->HasGroundParticleChanged();
			
			if (bShouldBeInGraph && !bIsInGraph)
			{
				IslandManager.AddConstraint(ContainerId, Constraint, Constraint->GetConstrainedParticles());
			}
			else if (bIsInGraph && !bShouldBeInGraph)
			{
				IslandManager.RemoveConstraint(Constraint);
			}
			else if (bIsInGraph && bShouldUpdateGraph)
			{
				IslandManager.RemoveConstraint(Constraint);
				IslandManager.AddConstraint(ContainerId, Constraint, Constraint->GetConstrainedParticles());
			}
		}
	}

	void FCharacterGroundConstraintContainer::PrepareTick()
	{
	}

	void FCharacterGroundConstraintContainer::UnprepareTick()
	{
	}

	void FCharacterGroundConstraintContainer::DisconnectConstraints(const TSet<TGeometryParticleHandle<FReal, 3>*>& RemovedParticles)
	{
		for (TGeometryParticleHandle<FReal, 3>*RemovedParticle : RemovedParticles)
		{
			TArray<FCharacterGroundConstraintHandle*> ConstraintsWithRemovedGroundBody;
			for (FConstraintHandle* ConstraintHandle : RemovedParticle->ParticleConstraints())
			{
				if (FCharacterGroundConstraintHandle* Constraint = ConstraintHandle->As<FCharacterGroundConstraintHandle>())
				{
					if (Constraint->IsValid())
					{
						// If it's the character particle remove the constraint
						// If it's the ground particle keep the constraint and set the ground particle to null
						if (Constraint->CharacterParticle == RemovedParticle)
						{
							Constraint->SetEnabled(false); // constraint lifespan is managed by the proxy
							Constraint->CharacterParticle = nullptr;
						}
						else if (Constraint->GroundParticle == RemovedParticle)
						{
							ConstraintsWithRemovedGroundBody.Add(Constraint);
						}
					}
					else
					{
						Constraint->SetEnabled(false);
					}
				}
			}

			for (FCharacterGroundConstraintHandle* Constraint : ConstraintsWithRemovedGroundBody)
			{
				Constraint->SetGroundParticle(nullptr);
			}
		}
	}

	void FCharacterGroundConstraintContainer::OnDisableParticle(FGeometryParticleHandle* DisabledParticle)
	{
		// Only disable the constraint if it's the character particle being disabled.
		// If the ground particle is disabled set it to null. This will remove the constraint
		// from the disabled particle's constraint list, so if the ground particle is re-enabled
		// it won't automatically be set as the ground particle in the constraint
		for (FCharacterGroundConstraintHandle* Constraint : GetConstraints())
		{
			if (Constraint->IsValid())
			{
				if (Constraint->GetCharacterParticle() == DisabledParticle)
				{
					if (Constraint->IsEnabled())
					{
						Constraint->SetEnabled(false);
					}
				}
				else if (Constraint->GetGroundParticle() == DisabledParticle)
				{
					Constraint->SetGroundParticle(nullptr);
				}
			}
			else
			{
				Constraint->SetEnabled(false);
			}
		}
	}

	void FCharacterGroundConstraintContainer::OnEnableParticle(FGeometryParticleHandle* EnabledParticle)
	{
		for (FConstraintHandle* ConstraintHandle : EnabledParticle->ParticleConstraints())
		{
			if (FCharacterGroundConstraintHandle* Constraint = ConstraintHandle->As<FCharacterGroundConstraintHandle>())
			{
				if ((Constraint->GetCharacterParticle() == EnabledParticle) && !Constraint->IsEnabled())
				{
					Constraint->SetEnabled(true);
				}
			}
		}
	}

	bool FCharacterGroundConstraintContainer::CanEvaluate(const FCharacterGroundConstraintHandle* Constraint) const
	{
		if (!Constraint->IsEnabled())
		{
			return false;
		}

		const FGenericParticleHandle CharacterParticle = FGenericParticleHandle(Constraint->GetCharacterParticle());

		// check for valid and enabled character particle
		if (CharacterParticle->Handle() == nullptr || CharacterParticle->Disabled() || CharacterParticle->Sleeping() || !(CharacterParticle->InvM() > 0.0))
		{
			return false;
		}

		return true;
	}

}