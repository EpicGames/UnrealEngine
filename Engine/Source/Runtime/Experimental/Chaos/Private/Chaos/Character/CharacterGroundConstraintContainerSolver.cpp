// Copyright Epic Games, Inc. All Rights Reserved.

#include "CharacterGroundConstraintContainerSolver.h"
#include "Chaos/Island/IslandManager.h"

namespace Chaos
{
	namespace Private
	{
		FCharacterGroundConstraintContainerSolver::FCharacterGroundConstraintContainerSolver(FCharacterGroundConstraintContainer& InConstraintContainer, const int32 InPriority)
			: FConstraintContainerSolver(InPriority)
			, ConstraintContainer(InConstraintContainer)
		{
		}

		FCharacterGroundConstraintContainerSolver::~FCharacterGroundConstraintContainerSolver()
		{
		}

		void FCharacterGroundConstraintContainerSolver::ResetImpl(const int32 InMaxConstraints)
		{
			SolvedConstraints.Reset(InMaxConstraints);

			Solvers.Empty();
			Solvers.SetNum(InMaxConstraints);
		}

		void FCharacterGroundConstraintContainerSolver::AddConstraints()
		{
			Reset(ConstraintContainer.GetNumConstraints());

			for (FCharacterGroundConstraintHandle* Constraint : ConstraintContainer.GetConstraints())
			{
				AddConstraint(Constraint);
			}
		}

		void FCharacterGroundConstraintContainerSolver::AddConstraints(const TArrayView<Private::FPBDIslandConstraint*>& IslandConstraints)
		{
			for (Private::FPBDIslandConstraint* IslandConstraint : IslandConstraints)
			{
				// Filter out sleeping constraints in any partially sleeping island
				if (!IslandConstraint->IsSleeping())
				{
					FConstraintHandle* Constraint = IslandConstraint->GetConstraint();

					// We will only ever be given constraints from our container (asserts in non-shipping)
					FCharacterGroundConstraintHandle* CharacterGroundConstraint = Constraint->AsUnsafe<FCharacterGroundConstraintHandle>();

					AddConstraint(CharacterGroundConstraint);
				}
			}
		}

		void FCharacterGroundConstraintContainerSolver::AddConstraint(FCharacterGroundConstraintHandle* Constraint)
		{
			// If this triggers, Reset was called with the wrong constraint count
			check(SolvedConstraints.Num() < SolvedConstraints.Max());

			SolvedConstraints.Add(Constraint);
		}

		bool FCharacterGroundConstraintContainerSolver::SortConstraintsByPartitionIndex(const TArray<int32>& PartitionIndices, const TArray<int32>& PrefixSum)
		{
			Private::SortArrayByPartitionIndex(SolvedConstraints, PartitionIndices, PrefixSum);
			return true;
		}

		void FCharacterGroundConstraintContainerSolver::GetAllConstrainedParticles(TArray<const FGeometryParticleHandle*>& ParticleHandles) const
		{
			ParticleHandles.Reset();
			ParticleHandles.Reserve(SolvedConstraints.Num() * 2);

			for (const FCharacterGroundConstraintHandle* Constraint : SolvedConstraints)
			{
				ParticleHandles.Add(Constraint->GetCharacterParticle());
				ParticleHandles.Add(Constraint->GetGroundParticle());
			}
		}

		void FCharacterGroundConstraintContainerSolver::AddBodies(FSolverBodyContainer& SolverBodyContainer)
		{
			for (int32 SolverConstraintIndex = 0, SolverConstraintEndIndex = SolvedConstraints.Num(); SolverConstraintIndex < SolverConstraintEndIndex; ++SolverConstraintIndex)
			{
				FCharacterGroundConstraintHandle* Constraint = SolvedConstraints[SolverConstraintIndex];

				FGenericParticleHandle CharacterParticle = FGenericParticleHandle(Constraint->GetCharacterParticle());
				FGenericParticleHandle GroundParticle = FGenericParticleHandle(Constraint->GetGroundParticle());

				FSolverBody* CharacterSolverBody = SolverBodyContainer.FindOrAdd(CharacterParticle);

				if (GroundParticle.IsValid())
				{
					FSolverBody* GroundSolverBody = SolverBodyContainer.FindOrAdd(GroundParticle);
					Solvers[SolverConstraintIndex].SetBodies(CharacterSolverBody, GroundSolverBody);
				}
				else
				{
					Solvers[SolverConstraintIndex].SetBodies(CharacterSolverBody);
				}
			}
		}

		void FCharacterGroundConstraintContainerSolver::GatherInput(const FReal Dt)
		{
			GatherInput(Dt, 0, SolvedConstraints.Num());
		}

		void FCharacterGroundConstraintContainerSolver::GatherInput(const FReal Dt, const int32 BeginIndex, const int32 EndIndex)
		{
			// We may have conservatively allocated the solver arrays. If so, reduce their size now
			const int32 NumConstraints = GetNumConstraints();
			check(Solvers.Num() >= NumConstraints);
			if (Solvers.Num() > NumConstraints)
			{
				Solvers.SetNum(GetNumConstraints());
			}

			for (int32 SolverConstraintIndex = BeginIndex; SolverConstraintIndex < EndIndex; ++SolverConstraintIndex)
			{
				FCharacterGroundConstraintHandle* Constraint = SolvedConstraints[SolverConstraintIndex];
				Solvers[SolverConstraintIndex].GatherInput(Dt, Constraint->GetSettings(), Constraint->GetData());
			}
		}

		void FCharacterGroundConstraintContainerSolver::ScatterOutput(const FReal Dt)
		{
			ScatterOutput(Dt, 0, SolvedConstraints.Num());
		}

		void FCharacterGroundConstraintContainerSolver::ScatterOutput(const FReal Dt, const int32 BeginIndex, const int32 EndIndex)
		{
			for (int32 SolverConstraintIndex = BeginIndex; SolverConstraintIndex < EndIndex; ++SolverConstraintIndex)
			{
				FVec3 SolverAppliedForce, SolverAppliedTorque;
				Solvers[SolverConstraintIndex].ScatterOutput(Dt, SolverAppliedForce, SolverAppliedTorque);
				FCharacterGroundConstraintHandle* Constraint = SolvedConstraints[SolverConstraintIndex];
				Constraint->SolverAppliedForce = SolverAppliedForce;
				Constraint->SolverAppliedTorque = SolverAppliedTorque;
			}
		}

		void FCharacterGroundConstraintContainerSolver::ApplyPositionConstraints(const FReal Dt, const int32 It, const int32 NumIts)
		{
			check(Solvers.Num() <= GetNumConstraints());
			ApplyPositionConstraints(Dt, It, NumIts, 0, Solvers.Num());
		}

		void FCharacterGroundConstraintContainerSolver::ApplyPositionConstraints(const FReal Dt, const int32 It, const int32 NumIts, const int32 ConstraintBeginIndex, const int32 ConstraintEndIndex)
		{
			if (ConstraintEndIndex <= ConstraintBeginIndex)
			{
				return;
			}

			check(Solvers.Num() <= GetNumConstraints());
			// NOTE: may be called in parallel. Should not change the container or any elements outside of [BeginIndex, EndIndex)
			check(ConstraintBeginIndex >= 0);
			check(ConstraintEndIndex <= GetNumConstraints());

			// Handle the case where we dropped some constraints because there were too  many
			const int32 BeginIndex = FMath::Min(ConstraintBeginIndex, Solvers.Num());
			const int32 EndIndex = FMath::Min(ConstraintEndIndex, Solvers.Num());

			for (int32 SolverConstraintIndex = BeginIndex; SolverConstraintIndex < EndIndex; ++SolverConstraintIndex)
			{
				Solvers[SolverConstraintIndex].SolvePosition();
			}
		}

		void FCharacterGroundConstraintContainerSolver::ApplyVelocityConstraints(const FReal Dt, const int32 It, const int32 NumIts)
		{
		}

		void FCharacterGroundConstraintContainerSolver::ApplyVelocityConstraints(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex)
		{
		}

		void FCharacterGroundConstraintContainerSolver::ApplyProjectionConstraints(const FReal Dt, const int32 It, const int32 NumIts)
		{
		}

		void FCharacterGroundConstraintContainerSolver::ApplyProjectionConstraints(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex)
		{
		}

	} // namespace Private
} // namespace Chaos