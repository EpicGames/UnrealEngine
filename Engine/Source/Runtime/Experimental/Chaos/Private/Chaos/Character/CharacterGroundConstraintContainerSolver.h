// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Evolution/SolverConstraintContainer.h"
#include "Chaos/Character/CharacterGroundConstraintContainer.h"
#include "CharacterGroundConstraintSolver.h"

namespace Chaos
{
	namespace Private
	{
		class FCharacterGroundConstraintContainerSolver : public FConstraintContainerSolver
		{
		public:
			FCharacterGroundConstraintContainerSolver(FCharacterGroundConstraintContainer& InConstraintContainer, const int32 InPriority);
			~FCharacterGroundConstraintContainerSolver();

			//////////////////////////////////////////////////////////////////////////
			// FConstraintContainerSolver impl

			virtual int32 GetNumConstraints() const override final { return SolvedConstraints.Num(); }

			/// Add all constraints from the constraint container in the order that they appear there
			virtual void AddConstraints() override final;

			/// Add all constraints from an island (guaranteed to be from the same constraint container)
			virtual void AddConstraints(const TArrayView<Private::FPBDIslandConstraint*>& IslandConstraints) override final;

			/// Set the bodies on the constraint from the constraint data solver
			virtual void AddBodies(FSolverBodyContainer& SolverBodyContainer) override final;

			/// Initialize the solver from the constraint data
			virtual void GatherInput(const FReal Dt) override final;
			virtual void GatherInput(const FReal Dt, const int32 BeginIndex, const int32 EndIndex) override final;

			/// Get the solver output that will be stored per constraint - applied force and torque
			virtual void ScatterOutput(const FReal Dt) override final;
			virtual void ScatterOutput(const FReal Dt, const int32 BeginIndex, const int32 EndIndex) override final;

			/// Apply a single position solve iteration for all constraints
			virtual void ApplyPositionConstraints(const FReal Dt, const int32 It, const int32 NumIts) override final;
			virtual void ApplyPositionConstraints(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex) override final;

			/// Apply a single velocity solve iteration for all constraints
			/// Note: Currently not used for character ground constraints
			virtual void ApplyVelocityConstraints(const FReal Dt, const int32 It, const int32 NumIts) override final;
			virtual void ApplyVelocityConstraints(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex) override final;

			/// Apply a single projection solve iteration for all constraints
			/// Note: Currently not used for character ground constraints
			virtual void ApplyProjectionConstraints(const FReal Dt, const int32 It, const int32 NumIts) override final;
			virtual void ApplyProjectionConstraints(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex) override final;

		private:
			virtual void ResetImpl(const int32 InMaxConstraints) override final;
			virtual bool SupportsPartitioning() const override final { return true; }
			virtual bool SortConstraintsByPartitionIndex(const TArray<int32>& PartitionIndices, const TArray<int32>& PrefixSum) override final;
			virtual void GetAllConstrainedParticles(TArray<const FGeometryParticleHandle*>& ParticleHandles) const override final;

			void AddConstraint(FCharacterGroundConstraintHandle* Constraint);

			FCharacterGroundConstraintContainer& ConstraintContainer;

			TArray<FCharacterGroundConstraintHandle*> SolvedConstraints;

			// Array of single constraint solvers
			TArray<FCharacterGroundConstraintSolver> Solvers;
		};

	} // namespace Private
} // namespace Chaos