// Copyright Epic Games, Inc. All Rights Reserved.

#include "CharacterGroundConstraintSolver.h"

namespace Chaos
{
	namespace Private
	{
		static FRotation3 GetRelativeRotation(const FRotation3& From, const FRotation3& To)
		{
			FReal DP = FRotation3::DotProduct(From, To);
			FRotation3 RelRot = To * From.Inverse();
			return DP < FReal(0.0f) ? -RelRot : RelRot;
		}

		//////////////////////////////////////////////////////////////////////////
		/// FBodyData

		FCharacterGroundConstraintSolver::FBodyData::FBodyData()
		{
			// Initializes the additional state data
			// These modifiers are not currently being used, so initialize them once to their default values
			CharacterBody.Init();
			GroundBody.Init();
		}

		void FCharacterGroundConstraintSolver::FBodyData::Init(FSolverBody* InCharacterBody, FSolverBody* InGroundBody)
		{
			check(InCharacterBody != nullptr);
			CharacterBody.SetSolverBody(*InCharacterBody);

			if (InGroundBody != nullptr)
			{
				GroundBody.SetSolverBody(*InGroundBody);
			}
		}

		void FCharacterGroundConstraintSolver::FBodyData::Reset()
		{
			CharacterBody.Reset();
			GroundBody.Reset();
		}

		//////////////////////////////////////////////////////////////////////////
		/// FImpulseData

		FCharacterGroundConstraintSolver::FImpulseData::FImpulseData()
		{
			Reset();
		}

		void FCharacterGroundConstraintSolver::FImpulseData::Reset()
		{
			LinearPositionImpulse = FSolverVec3::ZeroVector;
			AngularSwingImpulse = FSolverVec3::ZeroVector;
			AngularImpulse = FSolverReal(0.0f);
			LinearCorrectionImpulse = FSolverReal(0.0f);
		}

		//////////////////////////////////////////////////////////////////////////
		/// FConstraintData

		FCharacterGroundConstraintSolver::FConstraintData::FConstraintData()
			: Normal(FSolverVec3(0.0f, 0.0f, 1.0f)) // Need to initialize normal as it is used in GetLinearImpulse
			, VerticalAxis(FSolverVec3(0.0f, 0.0f, 1.0f)) // Need to vertical axis as it is used in GetAngularImpulse
			, CharacterInvM(-1.0f)
		{
		}

		bool FCharacterGroundConstraintSolver::FConstraintData::IsValid()
		{
			return CharacterInvM > 0.0f;
		}

		//////////////////////////////////////////////////////////////////////////
		/// FCharacterGroundConstraintSolver

		void FCharacterGroundConstraintSolver::SetBodies(FSolverBody* CharacterSolverBody, FSolverBody* GroundSolverBody)
		{
			BodyData.Init(CharacterSolverBody, GroundSolverBody);
		}

		void FCharacterGroundConstraintSolver::Reset()
		{
			BodyData.Reset();
		}

		void FCharacterGroundConstraintSolver::GatherInput(FReal Dt, const FCharacterGroundConstraintSettings& Settings, const FCharacterGroundConstraintDynamicData& Data)
		{
			ImpulseData.Reset();

			//////////////////////////////////////////////////////////////////////////
			// Mass conditioning

			FReal CharacterInvM = BodyData.CharacterBody.InvM();
			FSolverVec3 CharacterInvILocal = BodyData.CharacterBody.InvILocal();

			FReal GroundInvM = 0.0;

			const bool bTwoBody = BodyData.GroundBody.IsValid();
			if (bTwoBody && Settings.MaxCharacterGroundMassRatio > 0.0f && BodyData.GroundBody.InvM() > 0.0f)
			{
				GroundInvM = BodyData.GroundBody.InvM();
				const FReal Ratio = GroundInvM / CharacterInvM;
				if (Ratio > Settings.MaxCharacterGroundMassRatio)
				{
					const FReal OldInvM = CharacterInvM;
					CharacterInvM = GroundInvM / Settings.MaxCharacterGroundMassRatio;
					CharacterInvILocal *= FSolverReal(CharacterInvM / OldInvM);
				}
			}

			//////////////////////////////////////////////////////////////////////////
			// Normal constraint

			const FVec3 Character_X = BodyData.CharacterBody.X();
			const FVec3 Character_P = BodyData.CharacterBody.P();
			const FRotation3 Character_R(BodyData.CharacterBody.R());
			const FRotation3 Character_Q(BodyData.CharacterBody.Q());

			const FVec3 PC_Init = Character_X - FVec3(Settings.TargetHeight * Settings.VerticalAxis);
			const FVec3 PG_Init = Character_X - FVec3(Data.GroundDistance * Settings.VerticalAxis);
			const FRotation3 RelQuatCharacter = GetRelativeRotation(Character_R, Character_Q);
			const FReal CharacterAngularDisplacement = RelQuatCharacter.ToRotationVector().Dot(Settings.VerticalAxis);
			const FVec3 CharacterLinearDisplacement = FVec3(Character_P - Character_X);

			FVec3 InitialError = FVec3(PC_Init - PG_Init);
			FVec3 ProjectedError = InitialError + CharacterLinearDisplacement;

			FReal TwoBodyEffectiveMass = 1.0 / CharacterInvM;
			
			FMatrix33 GroundInvI(0.0);
			FVec3 GroundNormal = Data.GroundNormal;
			FVec3 GroundOffset = FVec3(ForceInitToZero);

			FRotation3 RelQuatGround = FRotation3::Identity;
			FVec3 GroundLinearDisplacement(ForceInitToZero);
			FVec3 Ground_X(ForceInitToZero);
			FVec3 Ground_P(ForceInitToZero);
			FRotation3 Ground_Q(FRotation3::Identity);

			if (bTwoBody)
			{
				Ground_X = BodyData.GroundBody.X();
				Ground_P = BodyData.GroundBody.P();
				Ground_Q = FRotation3(BodyData.GroundBody.Q());
				const FRotation3 Ground_R(BodyData.GroundBody.R());
				RelQuatGround = GetRelativeRotation(Ground_R, Ground_Q);

				const FVec3 RC_Init = FVec3(PC_Init - Ground_X);
				const FVec3 RG_Init = FVec3(PG_Init - Ground_X);

				GroundLinearDisplacement = FVec3(Ground_P - Ground_X) + RelQuatGround * RG_Init - RG_Init;
				GroundOffset = PC_Init + CharacterLinearDisplacement - Ground_P;
				GroundNormal = RelQuatGround * GroundNormal;
				ProjectedError -= GroundLinearDisplacement;

				GroundInvM = BodyData.GroundBody.InvM();
				GroundInvI = Utilities::ComputeWorldSpaceInertia(Ground_Q, BodyData.GroundBody.InvILocal());
				const FVec3 R_Cross_N = GroundOffset.Cross(GroundNormal);
				TwoBodyEffectiveMass = 1.0f / (CharacterInvM + GroundInvM + R_Cross_N.Dot(GroundInvI * R_Cross_N));
			}

			// Vertical damping
			const FReal ProjectedOverlap = -ProjectedError.Dot(GroundNormal);
			const bool WillOverlap = ProjectedOverlap > 0.0;

			const FReal InitialOverlap = -InitialError.Dot(GroundNormal);
			const bool CurrentlyOverlapping = InitialOverlap > 0.0f;

			const bool NeedsCorrection = CurrentlyOverlapping && WillOverlap;
			CorrectionSolveFunction = NeedsCorrection ? &SolveCorrectionSingleBody : &NoSolve;

			if (CurrentlyOverlapping)
			{
				FReal DampingFactor = FMath::Clamp(Settings.DampingFactor, 0.0, 1.0);
				FVec3 Diff = DampingFactor * InitialOverlap * GroundNormal;
				InitialError += Diff;
				ProjectedError += Diff;
			}

			// Write constraint data
			ConstraintData.CharacterInvM = FSolverReal(CharacterInvM);
			ConstraintData.GroundInvM = FSolverReal(GroundInvM);
			ConstraintData.GroundInvI = FSolverMatrix33(GroundInvI);
			ConstraintData.EffectiveMassN = FSolverReal(TwoBodyEffectiveMass);
			ConstraintData.Normal = FSolverVec3(GroundNormal);
			ConstraintData.GroundOffset = FSolverVec3(GroundOffset);
			ConstraintData.InitialError = FSolverReal(InitialError.Dot(GroundNormal));
			ConstraintData.InitialProjectedError = FSolverReal(ProjectedError.Dot(GroundNormal));

			//////////////////////////////////////////////////////////////////////////
			// Upright constraint
			const Chaos::FMatrix33 CharacterInvInertia = Utilities::ComputeWorldSpaceInertia(Character_Q, CharacterInvILocal);
			ConstraintData.CharacterInvI = FSolverMatrix33(CharacterInvInertia);
			ConstraintData.VerticalAxis = FSolverVec3(Settings.VerticalAxis);
			ConstraintData.CharacterVerticalAxis = FSolverVec3(Character_Q.RotateVector(FVector::UpVector));

			//////////////////////////////////////////////////////////////////////////
			// Motion Target

			FVec3 TargetDeltaPos = Data.TargetDeltaPosition;
			FReal TargetDeltaFacing = Data.TargetDeltaFacing;

			if (BodyData.GroundBody.IsValid())
			{
				TargetDeltaPos = GroundLinearDisplacement + RelQuatGround * TargetDeltaPos;
				TargetDeltaFacing += RelQuatGround.ToRotationVector().Dot(Settings.VerticalAxis);
			}

			// If the slope is too steep adjust the target to not point up the slope
			// and set the radial force to zero
			const FReal DtSq = Dt * Dt;
			bool bSliding = false;
			FReal DP = FVec3::DotProduct(Settings.VerticalAxis, GroundNormal);
			if (DP <= Data.CosMaxWalkableSlopeAngle)
			{
				FVec3 UpSlope = Settings.VerticalAxis - DP * GroundNormal;
				UpSlope.Normalize();
				FReal UpMotion = TargetDeltaPos.Dot(UpSlope);
				if (UpMotion > FReal(0.0))
				{
					TargetDeltaPos -= UpMotion * FVec3(UpSlope);
				}
				ConstraintData.RadialImpulseLimit = FSolverReal(Settings.FrictionForceLimit * DtSq);
				bSliding = true;
			}
			else
			{
				ConstraintData.RadialImpulseLimit = FSolverReal(Settings.RadialForceLimit * DtSq);
			}

			const FVec3 MotionTarget = Character_X + TargetDeltaPos;
			const FVec3 MotionTargetError = FVec3(Character_P - MotionTarget);
			const FReal MotionTargetAngularError = CharacterAngularDisplacement - TargetDeltaFacing;

			FReal MotionTargetErrorSq = MotionTargetError.SizeSquared();
			FReal MotionTargetDirSizeSq = MotionTargetErrorSq;
			FVec3 MotionTargetDir = FVec3::ForwardVector;
			if (MotionTargetDirSizeSq > UE_SMALL_NUMBER)
			{
				MotionTargetDir = MotionTargetError * FMath::InvSqrt(MotionTargetDirSizeSq);
			}
			else
			{
				MotionTargetDirSizeSq = TargetDeltaPos.SizeSquared();
				if (MotionTargetDirSizeSq > UE_SMALL_NUMBER)
				{
					MotionTargetDir = TargetDeltaPos * FMath::InvSqrt(MotionTargetDirSizeSq);
				}
			}

			ConstraintData.MassBiasT = FSolverReal(Settings.MotionTargetMassBias);

			FReal EffectiveMassT = 1.0 / CharacterInvM;
			FReal EffectiveInertiaT = 1.0 / Settings.VerticalAxis.Dot(CharacterInvInertia * Settings.VerticalAxis);
			if (bTwoBody)
			{
				const FVec3 R_Cross_T = GroundOffset.Cross(MotionTargetDir);
				EffectiveMassT = 1.0 / (CharacterInvM + ConstraintData.MassBiasT * (GroundInvM + R_Cross_T.Dot(GroundInvI * R_Cross_T)));
			}

			// Scale radial force limit to hit the motion target
			if (!bSliding && (MotionTargetErrorSq > UE_SMALL_NUMBER))
			{
				const FSolverReal MotionTargetImpulse = FSolverReal(EffectiveMassT * Settings.RadialForceMotionTargetScaling * FMath::Sqrt(MotionTargetErrorSq));
				ConstraintData.RadialImpulseLimit = ConstraintData.RadialImpulseLimit < MotionTargetImpulse ? MotionTargetImpulse : ConstraintData.RadialImpulseLimit;
			}

			// Write constraint data
			ConstraintData.AngularTwistImpulseLimit = FSolverReal(Settings.TwistTorqueLimit * DtSq);
			ConstraintData.AngularSwingImpulseLimit = FSolverReal(Settings.SwingTorqueLimit * DtSq);
			ConstraintData.AssumedOnGroundHeight = FSolverReal(Settings.AssumedOnGroundHeight);
			ConstraintData.MotionTargetError = FSolverVec3(MotionTargetError);
			ConstraintData.MotionTargetAngularError = FSolverReal(MotionTargetAngularError);
			ConstraintData.EffectiveMassT = FSolverReal(EffectiveMassT);

			FSolverVec3 FacingConstraintRotationAxis = FSolverVec3(Settings.VerticalAxis);
			FacingConstraintRotationAxis.Normalize();

			ConstraintData.EffectiveInertiaT = FSolverReal(EffectiveInertiaT);

			PositionSolveFunction = bTwoBody ? &SolvePositionTwoBody : SolvePositionSingleBody;
		}

		void FCharacterGroundConstraintSolver::ScatterOutput(FReal Dt, FVec3& OutSolverAppliedForce, FVec3& OutSolverAppliedTorque)
		{
			if (Dt > UE_SMALL_NUMBER)
			{
				FReal FreqSq = 1.0 / (Dt * Dt);
				OutSolverAppliedForce = FVec3(ImpulseData.LinearPositionImpulse + ImpulseData.LinearCorrectionImpulse * ConstraintData.Normal) * FreqSq;
				OutSolverAppliedTorque = FVec3(ImpulseData.AngularImpulse * ConstraintData.VerticalAxis + ImpulseData.AngularSwingImpulse) * FreqSq;
			}
			else
			{
				OutSolverAppliedForce = FVec3::ZeroVector;
				OutSolverAppliedTorque = FVec3::ZeroVector;
			}
			Reset();

		}

	} // namespace Private
} // namespace Chaos