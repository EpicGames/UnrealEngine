// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "Chaos/Evolution/SolverBody.h"
#include "Chaos/ParticleHandleFwd.h"
#include "Chaos/Character/CharacterGroundConstraintSettings.h"
#include "Chaos/Utilities.h"

namespace Chaos
{
	namespace Private
	{
		//////////////////////////////////////////////////////////////////////////
		/// FCharacterGroundConstraintSolver class
		
		/// Computes the and applies linear and angular displacement for a character ground constraint
		class FCharacterGroundConstraintSolver
		{
		public:
			/// Must call SetBodies and GatherInput before solve
			void SetBodies(FSolverBody* CharacterSolverBody, FSolverBody* GroundSolverBody = nullptr);
			void GatherInput(FReal Dt, const FCharacterGroundConstraintSettings& Settings, const FCharacterGroundConstraintDynamicData& Data);

			/// Solve function performs one iteration of the solver
			void SolvePosition();

			/// Gets the solver output as a force and torque in units ML/T^2 and ML^2/T^2 respectively
			/// and resets the solver
			void ScatterOutput(const FReal Dt, FVec3& OutSolverAppliedForce, FVec3& OutSolverAppliedTorque);

			void Reset();

			/// Gets the solver linear displacement for this constraint and converts to an impulse in units of ML/T
			FVec3 GetLinearImpulse(FReal Dt) const;

			/// Gets the solver angular displacement for this constraint and converts to an impulse in units of ML/T
			FVec3 GetAngularImpulse(FReal Dt) const;


		private:
			/// Utility functions
			static FSolverVec3 ProjectOntoPlane(const FSolverVec3& Vector, const FSolverVec3& PlaneNormal);
			static FSolverVec3 ClampMagnitude(const FSolverVec3& Vector, const FSolverReal& Max);
			static FSolverReal ClampAbs(const FSolverReal& Value, const FSolverReal& Max);

			/// FBodyData
			struct FBodyData
			{
				FBodyData();
				void Init(FSolverBody* InCharacterBody, FSolverBody* InGroundBody);
				bool IsTwoBody();
				void Reset();

				FConstraintSolverBody CharacterBody;
				FConstraintSolverBody GroundBody;
			} BodyData;

			/// FImpulseData
			struct FImpulseData
			{
				FImpulseData();
				void Reset();

				FSolverVec3 LinearPositionImpulse;
				FSolverVec3 AngularSwingImpulse;
				FSolverReal AngularImpulse;
				FSolverReal LinearCorrectionImpulse;
			} ImpulseData;

			/// Constraints
			struct FConstraintData
			{
				FConstraintData();
				bool IsValid();

				FSolverMatrix33 CharacterInvI; /// World space ground body inverse inertia
				FSolverMatrix33 GroundInvI; /// World space ground body inverse inertia
				FSolverVec3 GroundOffset; /// Offset vector from ground body CoM to the constraint position

				FSolverVec3 Normal; ///	Ground plane normal direction
				FSolverVec3 VerticalAxis; /// World space vertical axis
				FSolverVec3 CharacterVerticalAxis; /// Vertical axis rotated by the character initial rotation

				FSolverVec3 MotionTargetError; /// Projected constraint error for motion target constraint
				FSolverReal MotionTargetAngularError; // Projected angular error for facing constraint
				FSolverReal InitialError; /// Constraint error pre-integration
				FSolverReal InitialProjectedError; /// Projected constraint error post-integration
				
				FSolverReal CharacterInvM; /// Character inverse mass
				FSolverReal GroundInvM; /// Ground body inverse mass

				FSolverReal EffectiveMassN; // Effective mass for relative motion in the normal direction
				FSolverReal EffectiveMassT; // Effective mass for relative motion in the motion target direction
				FSolverReal EffectiveInertiaT; /// Effective angular mass to rotate the character to the target facing direction 

				FSolverReal MassBiasT; // Mass bias applied to the ground body for the motion target constraint
				FSolverReal MassBiasF; // Mass bias applied to the ground body for the facing direction angular constraint

				FSolverReal RadialImpulseLimit; /// Radial linear impulse from the solver is clamped to this limit
				FSolverReal AngularTwistImpulseLimit; /// Angular impulse from the solver is clamped to this limit
				FSolverReal AngularSwingImpulseLimit; /// Angular swing impulse from the solver is clamped to this limit

				FSolverReal AssumedOnGroundHeight; /// Height below which the character is assumed to be grounded
			} ConstraintData;

			using FSolveFunctionType = void (*)(const FConstraintData& ConstraintData, FBodyData& BodyData, FImpulseData& ImpulseData);
			FSolveFunctionType PositionSolveFunction;
			FSolveFunctionType CorrectionSolveFunction;

			static void SolveCorrectionSingleBody(const FConstraintData& ConstraintData, FBodyData& BodyData, FImpulseData& ImpulseData);
			static void SolvePositionSingleBody(const FConstraintData& ConstraintData, FBodyData& BodyData, FImpulseData& ImpulseData);
			static void SolvePositionTwoBody(const FConstraintData& ConstraintData, FBodyData& BodyData, FImpulseData& ImpulseData);
			static void NoSolve(const FConstraintData&, FBodyData&, FImpulseData&) {}
		};

		//////////////////////////////////////////////////////////////////////////
		/// FCharacterGroundConstraintSolver inline functions

		FORCEINLINE_DEBUGGABLE FVec3 FCharacterGroundConstraintSolver::GetLinearImpulse(FReal Dt) const
		{
			return Dt > UE_SMALL_NUMBER ? FVec3(ImpulseData.LinearPositionImpulse + ImpulseData.LinearCorrectionImpulse * ConstraintData.Normal) / Dt : FVec3::ZeroVector;
		}

		FORCEINLINE_DEBUGGABLE FVec3 FCharacterGroundConstraintSolver::GetAngularImpulse(FReal Dt) const
		{
			return Dt > UE_SMALL_NUMBER ? FVec3(ImpulseData.AngularImpulse * ConstraintData.VerticalAxis) / Dt : FVec3::ZeroVector;
		}

		//////////////////////////////////////////////////////////////////////////
		/// Utility functions

		FORCEINLINE_DEBUGGABLE FSolverVec3 FCharacterGroundConstraintSolver::ProjectOntoPlane(const FSolverVec3& Vector, const FSolverVec3& PlaneNormal)
		{
			return Vector - FSolverVec3::DotProduct(Vector, PlaneNormal) * PlaneNormal;
		}

		FORCEINLINE_DEBUGGABLE FSolverVec3 FCharacterGroundConstraintSolver::ClampMagnitude(const FSolverVec3& Vector, const FSolverReal& Max)
		{
			const FSolverReal MagSq = Vector.SizeSquared();
			const FSolverReal MaxSq = Max * Max;
			if (MagSq > MaxSq)
			{
				if (MaxSq > UE_SMALL_NUMBER)
				{
					return Vector * FMath::InvSqrt(MagSq) * Max;

				}
				else
				{
					return FSolverVec3::ZeroVector;
				}
			}
			else
			{
				return Vector;
			}
		}

		FORCEINLINE_DEBUGGABLE FSolverReal FCharacterGroundConstraintSolver::ClampAbs(const FSolverReal& Value, const FSolverReal& Max)
		{
			return Value > Max ? Max : (Value < -Max ? -Max : Value);
		}

		//////////////////////////////////////////////////////////////////////////
		/// FBodyData

		FORCEINLINE_DEBUGGABLE bool FCharacterGroundConstraintSolver::FBodyData::IsTwoBody()
		{
			return GroundBody.IsValid() && GroundBody.IsDynamic();
		}

		//////////////////////////////////////////////////////////////////////////
		/// Solve Functions

		FORCEINLINE_DEBUGGABLE void FCharacterGroundConstraintSolver::SolvePosition()
		{
			check(ConstraintData.IsValid());

			// Note: Solving these together as part of the same loop for now but
			// may be better to split and solve correction first for the whole
			// system before starting the displacement solver
			(*CorrectionSolveFunction)(ConstraintData, BodyData, ImpulseData);
			(*PositionSolveFunction)(ConstraintData, BodyData, ImpulseData);
		}

		FORCEINLINE_DEBUGGABLE void FCharacterGroundConstraintSolver::SolveCorrectionSingleBody(const FConstraintData& ConstraintData, FBodyData& BodyData, FImpulseData& ImpulseData)
		{
			constexpr FSolverReal Zero(0.0f);
			const FSolverReal Error = ConstraintData.Normal.Dot(BodyData.CharacterBody.CP()) + ConstraintData.InitialError;
			if (Error < Zero)
			{
				const FSolverReal Delta = -Error / ConstraintData.CharacterInvM;
				ImpulseData.LinearCorrectionImpulse += Delta;
				BodyData.CharacterBody.ApplyPositionCorrectionDelta(FSolverVec3(ConstraintData.CharacterInvM * Delta * ConstraintData.Normal));
			}
		}

		FORCEINLINE_DEBUGGABLE void FCharacterGroundConstraintSolver::SolvePositionSingleBody(const FConstraintData& ConstraintData, FBodyData& BodyData, FImpulseData& ImpulseData)
		{
			constexpr FSolverReal Zero(0.0f);
			constexpr FSolverReal SizeSqTolerance(UE_SMALL_NUMBER);

			// Normal
			const FSolverReal Error = ConstraintData.Normal.Dot(BodyData.CharacterBody.DP()) + ConstraintData.InitialProjectedError;
			if (Error < Zero)
			{
				const FSolverVec3 Delta = -(Error / ConstraintData.CharacterInvM) * ConstraintData.Normal;
				ImpulseData.LinearPositionImpulse += Delta;
				BodyData.CharacterBody.ApplyPositionDelta(FSolverVec3(ConstraintData.CharacterInvM * Delta));
			}

			// Angular constraint
			FSolverVec3 NewCharacterVerticalAxis = ConstraintData.CharacterVerticalAxis + BodyData.CharacterBody.DQ().Cross(ConstraintData.CharacterVerticalAxis);
			NewCharacterVerticalAxis.Normalize();
			const FSolverVec3 CrossProd = NewCharacterVerticalAxis.Cross(ConstraintData.VerticalAxis);
			const FSolverReal SizeSq = CrossProd.SizeSquared();
			if (SizeSq > SizeSqTolerance)
			{
				const FSolverVec3 AngAxis = CrossProd * FMath::InvSqrt(SizeSq);
				const FSolverReal AngResistance = FSolverReal(1.0f) / (ConstraintData.CharacterInvI * AngAxis).Dot(AngAxis);
				const FSolverVec3 NewSwingImpulse = ClampMagnitude(ImpulseData.AngularSwingImpulse + AngResistance * FMath::Asin(FMath::Sqrt(SizeSq)) * AngAxis, ConstraintData.AngularSwingImpulseLimit);
				const FSolverVec3 Delta = NewSwingImpulse - ImpulseData.AngularSwingImpulse;
				ImpulseData.AngularSwingImpulse = NewSwingImpulse;
				BodyData.CharacterBody.ApplyRotationDelta(FSolverVec3(ConstraintData.CharacterInvI * Delta));
			}

			const FSolverReal NormalImpulse = FSolverVec3::DotProduct(ImpulseData.LinearPositionImpulse, ConstraintData.Normal);
			const FSolverReal MinNormalImpulse(0.0f);
			if (((NormalImpulse + ImpulseData.LinearCorrectionImpulse) > MinNormalImpulse) || Error < ConstraintData.AssumedOnGroundHeight)
			{
				// Target Position
				const FSolverVec3 MotionTargetError = ProjectOntoPlane(BodyData.CharacterBody.DP() + ConstraintData.MotionTargetError, ConstraintData.Normal);
				const FSolverVec3 InitialMotionTargetImpulse = ProjectOntoPlane(ImpulseData.LinearPositionImpulse, ConstraintData.Normal);
				FSolverVec3 NewMotionTargetImpulse = ClampMagnitude(InitialMotionTargetImpulse - MotionTargetError / ConstraintData.CharacterInvM, ConstraintData.RadialImpulseLimit);
				const FSolverVec3 Delta = NewMotionTargetImpulse - InitialMotionTargetImpulse;
				ImpulseData.LinearPositionImpulse += Delta;
				BodyData.CharacterBody.ApplyPositionDelta(FSolverVec3(ConstraintData.CharacterInvM * Delta));

				// Target Rotation
				const FSolverReal MotionTargetAngularError = ConstraintData.MotionTargetAngularError + ConstraintData.VerticalAxis.Dot(BodyData.CharacterBody.DQ());
				const FSolverReal NewAngularImpulse = ClampAbs(ImpulseData.AngularImpulse - ConstraintData.EffectiveInertiaT * MotionTargetAngularError, ConstraintData.AngularTwistImpulseLimit);
				const FSolverReal AngularDelta = NewAngularImpulse - ImpulseData.AngularImpulse;
				ImpulseData.AngularImpulse += AngularDelta;
				BodyData.CharacterBody.ApplyRotationDelta(FSolverVec3(ConstraintData.CharacterInvI * AngularDelta * ConstraintData.VerticalAxis));
			}
		}

		FORCEINLINE_DEBUGGABLE void FCharacterGroundConstraintSolver::SolvePositionTwoBody(const FConstraintData& ConstraintData, FBodyData& BodyData, FImpulseData& ImpulseData)
		{
			constexpr FSolverReal Zero(0.0f);
			constexpr FSolverReal SizeSqTolerance(UE_SMALL_NUMBER);

			// Normal
			const FSolverReal Error = ConstraintData.Normal.Dot(BodyData.CharacterBody.DP() - BodyData.GroundBody.DP() - BodyData.GroundBody.DQ().Cross(ConstraintData.GroundOffset)) + ConstraintData.InitialProjectedError;
			if (Error < Zero)
			{
				const FSolverVec3 Delta = -ConstraintData.EffectiveMassN * Error * ConstraintData.Normal;
				ImpulseData.LinearPositionImpulse += Delta;
				BodyData.CharacterBody.ApplyPositionDelta(FSolverVec3(ConstraintData.CharacterInvM * Delta));
				BodyData.GroundBody.ApplyPositionDelta(FSolverVec3(-ConstraintData.GroundInvM * Delta));
				BodyData.GroundBody.ApplyRotationDelta(FSolverVec3(-ConstraintData.GroundInvI * ConstraintData.GroundOffset.Cross(Delta)));
			}

			// Angular constraint
			FSolverVec3 NewCharacterVerticalAxis = ConstraintData.CharacterVerticalAxis + BodyData.CharacterBody.DQ().Cross(ConstraintData.CharacterVerticalAxis);
			NewCharacterVerticalAxis.Normalize();
			const FSolverVec3 CrossProd = NewCharacterVerticalAxis.Cross(ConstraintData.VerticalAxis);
			const FSolverReal SizeSq = CrossProd.SizeSquared();
			if (SizeSq > SizeSqTolerance)
			{
				const FSolverVec3 AngAxis = CrossProd * FMath::InvSqrt(SizeSq);
				const FSolverReal AngResistance = FSolverReal(1.0f) / (ConstraintData.CharacterInvI * AngAxis).Dot(AngAxis);
				const FSolverVec3 NewSwingImpulse = ClampMagnitude(ImpulseData.AngularSwingImpulse + AngResistance * FMath::Asin(FMath::Sqrt(SizeSq)) * AngAxis, ConstraintData.AngularSwingImpulseLimit);
				const FSolverVec3 Delta = NewSwingImpulse - ImpulseData.AngularSwingImpulse;
				ImpulseData.AngularSwingImpulse = NewSwingImpulse;
				BodyData.CharacterBody.ApplyRotationDelta(FSolverVec3(ConstraintData.CharacterInvI * Delta));
			}

			const FSolverReal NormalImpulse = FSolverVec3::DotProduct(ImpulseData.LinearPositionImpulse, ConstraintData.Normal);
			const FSolverReal MinNormalImpulse(0.0f);
			if (((NormalImpulse + ImpulseData.LinearCorrectionImpulse) > MinNormalImpulse) || Error < ConstraintData.AssumedOnGroundHeight)
			{
				// Target Position
				const FSolverVec3 MotionTargetError = ProjectOntoPlane(BodyData.CharacterBody.DP() - BodyData.GroundBody.DP() - BodyData.GroundBody.DQ().Cross(ConstraintData.GroundOffset) + ConstraintData.MotionTargetError, ConstraintData.Normal);
				const FSolverVec3 InitialMotionTargetImpulse = ProjectOntoPlane(ImpulseData.LinearPositionImpulse, ConstraintData.Normal);
				FSolverVec3 NewMotionTargetImpulse = ClampMagnitude(InitialMotionTargetImpulse - ConstraintData.EffectiveMassT * MotionTargetError, ConstraintData.RadialImpulseLimit);
				const FSolverVec3 Delta = NewMotionTargetImpulse - InitialMotionTargetImpulse;
				ImpulseData.LinearPositionImpulse += Delta;
				BodyData.CharacterBody.ApplyPositionDelta(FSolverVec3(ConstraintData.CharacterInvM * Delta));
				BodyData.GroundBody.ApplyPositionDelta(FSolverVec3(-ConstraintData.MassBiasT * ConstraintData.GroundInvM * Delta));
				BodyData.GroundBody.ApplyRotationDelta(FSolverVec3(-ConstraintData.MassBiasT * ConstraintData.GroundInvI * ConstraintData.GroundOffset.Cross(Delta)));

				//FSolverVec3 MotionTargetError = ConstraintData.MotionTargetError + BodyData.CharacterBody.DP() - BodyData.GroundBody.DP();
				//MotionTargetError -= BodyData.GroundBody.DQ().Cross(MotionTargetError);
				//MotionTargetError = ProjectOntoPlane(MotionTargetError, ConstraintData.Normal);

				// Target Rotation
				const FSolverReal MotionTargetAngularError = ConstraintData.MotionTargetAngularError + ConstraintData.VerticalAxis.Dot(BodyData.CharacterBody.DQ() - BodyData.GroundBody.DQ());
				const FSolverReal NewAngularImpulse = ClampAbs(ImpulseData.AngularImpulse - ConstraintData.EffectiveInertiaT * MotionTargetAngularError, ConstraintData.AngularTwistImpulseLimit);
				const FSolverReal AngularDelta = NewAngularImpulse - ImpulseData.AngularImpulse;
				ImpulseData.AngularImpulse += AngularDelta;
				BodyData.CharacterBody.ApplyRotationDelta(FSolverVec3(ConstraintData.CharacterInvI * AngularDelta * ConstraintData.VerticalAxis));
			}
		}

	} // namespace Private
} // namespace Chaos