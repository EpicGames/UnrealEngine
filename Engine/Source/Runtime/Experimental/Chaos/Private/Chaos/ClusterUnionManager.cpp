// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/ClusterUnionManager.h"

#include "Chaos/CastingUtilities.h"
#include "Chaos/GeometryQueries.h"
#include "Chaos/ImplicitObjectUnion.h"
#include "Chaos/PBDRigidClustering.h"
#include "Chaos/PBDRigidClusteringAlgo.h"
#include "Chaos/PBDRigidsEvolution.h"
#include "Chaos/PBDRigidsEvolutionGBF.h"
#include "PhysicsProxy/GeometryCollectionPhysicsProxy.h"
#include "ChaosStats.h"
#include "RewindData.h"

namespace Chaos
{
	namespace
	{
		bool bChaosClusterUnionGenerateInterclusterEdges = true;
		FAutoConsoleVariableRef CVarChaosClusterUnionGenerateInterclusterEdges(
			TEXT("p.Chaos.ClusterUnion.GenerateInterclusterEdges"),
			bChaosClusterUnionGenerateInterclusterEdges,
			TEXT("Whether to generate intercluster edges automatically when adding to a cluster union (and remove them when removing from the cluster union).")
		);

		// @tmp: To be removed
		bool bChaosClusterUnionDoNotAddEmptyClusters = true;
		FAutoConsoleVariableRef CVarChaosClusterUnionDoNotAddEmptyClusters(
			TEXT("p.Chaos.ClusterUnion.DoNotAddEmptyClusters"),
			bChaosClusterUnionDoNotAddEmptyClusters,
			TEXT("Gating a risky bug fix.")
		);

		// @tmp: To be removed
		bool bFixPartialDestruction = true;
		FAutoConsoleVariableRef CVarChaosFixPartialDestruction(
			TEXT("p.Chaos.ClusterUnion.FixPartialDestruction"),
			bFixPartialDestruction,
			TEXT("Fix the partial destruction.")
		);

		// Aim is to prevent re-ordering of operatioins when performing a remove and add in the same frame
		bool bFixRemoveDuringSort = true;
		FAutoConsoleVariableRef CVarChaosFixRemoveDuringSort(
			TEXT("p.Chaos.ClusterUnion.FixRemoveDuringSort"),
			bFixRemoveDuringSort,
			TEXT("Fix remove to top during sort.")
		);

		FRigidTransform3 GetParticleRigidFrameInClusterUnion(FPBDRigidParticleHandle* Child, const FRigidTransform3& ClusterWorldTM)
		{
			FRigidTransform3 Frame = FRigidTransform3::Identity;

			if (FPBDRigidClusteredParticleHandle* ClusterChild = Child->CastToClustered(); ClusterChild && ClusterChild->IsChildToParentLocked())
			{
				Frame = ClusterChild->ChildToParent();
			}
			else
			{
				const FRigidTransform3 ChildWorldTM(Child->GetX(), Child->GetR());
				Frame = ChildWorldTM.GetRelativeTransform(ClusterWorldTM);
			}

			return Frame;
		}

		void AddParticleToConnectionGraph(FRigidClustering& Clustering, FClusterUnion& ClusterUnion, FPBDRigidParticleHandle* Particle)
		{
			if (!ClusterUnion.ChildParticles.IsEmpty() && ClusterUnion.Geometry->GetType() == ImplicitObjectType::Union)
			{
				constexpr FReal kAddThickness = 5.0;
				const FRigidTransform3 ClusterWorldTM(ClusterUnion.InternalCluster->GetX(), ClusterUnion.InternalCluster->GetR());

				// Use the acceleration structure of the cluster union itself to make finding overlaps easy.
				const FImplicitObjectUnion& ShapeUnion = ClusterUnion.Geometry->GetObjectChecked<FImplicitObjectUnion>();
				const FRigidTransform3 FromTransform = GetParticleRigidFrameInClusterUnion(Particle, ClusterWorldTM);
				FAABB3 ParticleLocalBounds = Particle->GetGeometry()->BoundingBox().TransformedAABB(FromTransform);
				ParticleLocalBounds.Thicken(kAddThickness);

				ShapeUnion.VisitOverlappingLeafObjects(
					ParticleLocalBounds,
					[&Clustering, &ClusterUnion, &FromTransform, Particle, kAddThickness](const FImplicitObject* ToGeom, const FRigidTransform3& ToTransform, const int32 RootObjectIndex, const int32 ObjectIndex, const int32 LeafObjectIndex)
					{
						check(ToGeom != nullptr);
						// RootObjectIndex is the index in the cluster union.
						// NOTE: This will need to be re-thought if we ever decide to make the mapping between child shapes and child particles not 1-to-1 (e.g. if we ever attempt to simplify the cluster union shape).

						// Ignore intersections against the same particle since we just added the particle (potentially) into the geometry.
						if ((RootObjectIndex >= ClusterUnion.ChildParticles.Num()) || ((RootObjectIndex < ClusterUnion.ChildParticles.Num()) && (Particle == ClusterUnion.ChildParticles[RootObjectIndex])))
						{
							return;
						}

						FAABB3 ToAABB = ToGeom->CalculateTransformedBounds(ToTransform);
						ToAABB.Thicken(kAddThickness);


						bool bOverlap = false;
						if (ToGeom->GetType() == ImplicitObjectType::LevelSet)
						{
							const FShapesArray& AllFromShapes = Particle->ShapesArray();
							for (int32 FromIndex = 0; FromIndex < AllFromShapes.Num(); ++FromIndex)
							{
								if (AllFromShapes[FromIndex])
								{
									const FPerShapeData& FromShape = *AllFromShapes[FromIndex];
									if (const FImplicitObject* FromGeom = FromShape.GetGeometry())
									{
										FAABB3 FromAABB = FromGeom->CalculateTransformedBounds(FromTransform);
										FromAABB.Thicken(kAddThickness);

										// First sanity check to see if the two shape AABB's intersect.
										if (FromAABB.Intersects(ToAABB))
										{
											// level set have missing implementation for some of the shapes at the moment 
											// for now we stop at the AABB test 
											bOverlap = true;
											break;
										}
									}
								}
							}
						}
						else
						{
							bOverlap = Utilities::CastHelper(*ToGeom, ToTransform,
								[Particle, &FromTransform, &ToAABB, kAddThickness](const auto& ToGeomDowncast, const FRigidTransform3& FinalToTransform)
								{
									const FShapesArray& AllFromShapes = Particle->ShapesArray();
									for (int32 FromIndex = 0; FromIndex < AllFromShapes.Num(); ++FromIndex)
									{
										if (!AllFromShapes[FromIndex])
										{
											continue;
										}

										const FPerShapeData& FromShape = *AllFromShapes[FromIndex];
										const FImplicitObject* FromGeom = FromShape.GetGeometry();

										if (!FromGeom)
										{
											continue;
										}

										FAABB3 FromAABB = FromGeom->CalculateTransformedBounds(FromTransform);
										FromAABB.Thicken(kAddThickness);

										// First sanity check to see if the two shape AABB's intersect.
										if (!FromAABB.Intersects(ToAABB))
										{
											continue;
										}

										// Now do a more accurate overlap check between the two shapes.
										// Note that passing MTD here is critical as it gets us a more accurate check for some reason...
										FMTDInfo MTDInfo;
										if (OverlapQuery(*FromGeom, FromTransform, ToGeomDowncast, FinalToTransform, kAddThickness, &MTDInfo))
										{
											return true;
										}
										else
										{
											continue;
										}
									}

									return false;
								});
						}
						if (bOverlap)
						{
							FPBDRigidParticleHandle* OtherParticle = ClusterUnion.ChildParticles[RootObjectIndex];
							Clustering.CreateNodeConnection(Particle, OtherParticle);
						}
					}
				);
			}
		}

		void RemoveClusterUnionEdges(FRigidClustering& Clustering, FClusterUnion& ClusterUnion, FPBDRigidParticleHandle* ParticleHandle)
		{
			if (!ParticleHandle)
			{
				return;
			}

			Clustering.RemoveNodeConnections(ParticleHandle);

			if (FClusterUnionParticleProperties* Properties = ClusterUnion.ChildProperties.Find(ParticleHandle))
			{
				Properties->bEdgesAreGenerated = false;
			}

			if (bChaosClusterUnionGenerateInterclusterEdges)
			{
				if (FPBDRigidClusteredParticleHandle* ClusterParticle = ParticleHandle->CastToClustered())
				{
					if (TArray<FPBDRigidParticleHandle*>* AllChildren = Clustering.GetChildrenMap().Find(ClusterParticle))
					{
						for (FPBDRigidParticleHandle* Child : *AllChildren)
						{
							if (!Child)
							{
								continue;
							}

							if (FPBDRigidClusteredParticleHandle* ClusterChild = Child->CastToClustered())
							{
								Clustering.RemoveFilteredNodeConnections(
									ClusterChild,
								    [Child](const FConnectivityEdge& Edge)
									{
										return IsInterclusterEdge(*Child, Edge);
									}
								);
							}
						}
					}
				}
			}
		}

		void GenerateConnectionGraph(FRigidClustering& Clustering, FClusterUnion& ClusterUnion)
		{
			for (FPBDRigidParticleHandle* ChildParticle : ClusterUnion.ChildParticles)
			{
				RemoveClusterUnionEdges(Clustering, ClusterUnion, ChildParticle);
				AddParticleToConnectionGraph(Clustering, ClusterUnion, ChildParticle);
			}
		}

		FPBDRigidClusteredParticleHandle* GetParentParticleInClusterUnion(const FClusterUnion& ClusterUnion, FPBDRigidClusteredParticleHandle* Particle)
		{
			if (!Particle)
			{
				return nullptr;
			}

			if (Particle->Parent() == ClusterUnion.InternalCluster)
			{
				return Particle;
			}

			return GetParentParticleInClusterUnion(ClusterUnion, Particle->Parent());
		}

		FGeometryCollectionPhysicsProxy* GetGCProxy(FPBDRigidParticleHandle* Particle)
		{
			if (IPhysicsProxyBase* Proxy = Particle->PhysicsProxy())
			{
				if (Proxy->GetType() == FGeometryCollectionPhysicsProxy::ConcreteType())
				{
					return static_cast<FGeometryCollectionPhysicsProxy*>(Proxy);
				}
			}
			return nullptr;
		}
	}

	const TArray<FPBDRigidParticleHandle*>& FClusterUnion::GetPendingGeometryOperationParticles(EClusterUnionGeometryOperation Op) const
	{
		static TArray<FPBDRigidParticleHandle*> EmptyArray;
		if (const TArray<FPBDRigidParticleHandle*>* Results = PendingGeometryOperations.Find(Op))
		{
			return *Results;
		}
		return EmptyArray;
	}

	void FClusterUnion::AddPendingGeometryOperation(EClusterUnionGeometryOperation Op, FPBDRigidParticleHandle* Particle)
	{
		PendingGeometryOperations.FindOrAdd(Op).AddUnique(Particle);
	}

	void FClusterUnion::ClearAllPendingGeometryOperations()
	{
		bGeometryModified = true;
		PendingGeometryOperations.Empty();
	}

	void FClusterUnion::ClearPendingGeometryOperations(EClusterUnionGeometryOperation Op)
	{
		bGeometryModified = true;
		PendingGeometryOperations.Remove(Op);
	}

	FClusterUnionManager::FClusterUnionManager(FRigidClustering& InClustering, FPBDRigidsEvolutionGBF& InEvolution)
		: MClustering(InClustering)
		, MEvolution(InEvolution)
	{
	}

	DECLARE_CYCLE_STAT(TEXT("FClusterUnionManager::CreateNewClusterUnion"), STAT_CreateNewClusterUnion, STATGROUP_Chaos);
	FClusterUnionIndex FClusterUnionManager::CreateNewClusterUnion(const FClusterCreationParameters& Parameters, const FClusterUnionCreationParameters& ClusterUnionParameters)
	{
		SCOPE_CYCLE_COUNTER(STAT_CreateNewClusterUnion);
		FClusterUnionIndex NewIndex = ClaimNextUnionIndex();
		check(NewIndex > 0);

		FClusterUnion NewUnion;
		NewUnion.InternalIndex = NewIndex;
		NewUnion.ExplicitIndex = ClusterUnionParameters.ExplicitIndex;
		NewUnion.Geometry = MakeImplicitObjectPtr<FImplicitObjectUnionClustered>();
		NewUnion.InternalCluster = MClustering.CreateClusterParticle(-NewIndex, {}, Parameters, NewUnion.Geometry, nullptr, ClusterUnionParameters.UniqueIndex);
		NewUnion.Parameters = Parameters;
		NewUnion.ClusterUnionParameters = ClusterUnionParameters;

		// Some parameters aren't relevant after creation.
		NewUnion.ClusterUnionParameters.UniqueIndex = nullptr;

		if (ensure(NewUnion.InternalCluster != nullptr))
		{
			if (ClusterUnionParameters.GravityGroupOverride != INDEX_NONE)
			{
				NewUnion.InternalCluster->SetGravityGroupIndex(ClusterUnionParameters.GravityGroupOverride);
			}

			NewUnion.InternalCluster->SetInternalCluster(true);
			if (AccelerationStructureSplitStaticAndDynamic == 1)
			{
				NewUnion.InternalCluster->SetSpatialIdx(FSpatialAccelerationIdx{ 0, 1 });
			}
			else
			{
				NewUnion.InternalCluster->SetSpatialIdx(FSpatialAccelerationIdx{ 0, 0 });
			}
			// No bounds for now since we don't have particles. When/if we do get particles later, updating the
			// geometry should switch this flag back on.
			NewUnion.InternalCluster->SetHasBounds(false);
		}
		MEvolution.DisableParticle(NewUnion.InternalCluster);

		ClusterUnions.Add(NewIndex, NewUnion);
		return NewIndex;
	}

	DECLARE_CYCLE_STAT(TEXT("FClusterUnionManager::DestroyClusterUnion"), STAT_DestroyClusterUnion, STATGROUP_Chaos);
	void FClusterUnionManager::DestroyClusterUnion(FClusterUnionIndex Index)
	{
		SCOPE_CYCLE_COUNTER(STAT_DestroyClusterUnion);

		if (FClusterUnion* ClusterUnion = FindClusterUnion(Index))
		{
			// Need to actually remove the particles and set them back into a simulatable state.
			// We need a clean removal here just in case the cluster union is actually being destroyed 
			// on the game thread prior to its children (which would live on another actor).
			// 
			// Note that we need to make a copy of the array here since the children list will be modified by the HandleRemoveOperation.
			// However, the function does not expect that the input array will change.
			TArray<FPBDRigidParticleHandle*> ChildrenCopy = ClusterUnion->ChildParticles;
			HandleRemoveOperation(Index, ChildrenCopy, EClusterUnionOperationTiming::Never);
			ClusterUnion->ChildParticles.Empty();
			MClustering.DestroyClusterParticle(ClusterUnion->InternalCluster);

			if (ClusterUnion->ExplicitIndex != INDEX_NONE)
			{
				ExplicitIndexMap.Remove(ClusterUnion->ExplicitIndex);
				PendingExplicitIndexOperations.Remove(ClusterUnion->ExplicitIndex);
			}
			ReusableIndices.Add(Index);
			PendingClusterIndexOperations.Remove(Index);
			ClusterUnions.Remove(Index);
		}
	}

	DECLARE_CYCLE_STAT(TEXT("FClusterUnionManager::ForceRecreateClusterUnionSharedGeometry"), STAT_ForceRecreateClusterUnionSharedGeometry, STATGROUP_Chaos);
	Chaos::FImplicitObjectPtr FClusterUnionManager::ForceRecreateClusterUnionGeometry(const FClusterUnion& Union)
	{
		SCOPE_CYCLE_COUNTER(STAT_ForceRecreateClusterUnionSharedGeometry);
		if (Union.ChildParticles.IsEmpty() || !Union.InternalCluster)
		{
			return MakeImplicitObjectPtr<FImplicitObjectUnionClustered>();
		}

		// TODO: Can we do something better than a union?
		const FRigidTransform3 ClusterWorldTM(Union.InternalCluster->GetX(), Union.InternalCluster->GetR());
		TArray<Chaos::FImplicitObjectPtr> Objects;
		Objects.Reserve(Union.ChildParticles.Num());

		for (FPBDRigidParticleHandle* Child : Union.ChildParticles)
		{
			const FRigidTransform3 Frame = GetParticleRigidFrameInClusterUnion(Child, ClusterWorldTM);
			if (Child->GetGeometry())
			{
				Objects.Add(Chaos::FImplicitObjectPtr(CreateTransformGeometryForClusterUnion<EThreadContext::Internal>(Child, Frame)));
			}
		}

		FImplicitObjectUnion* NewGeometry = new FImplicitObjectUnion(MoveTemp(Objects));
		NewGeometry->SetAllowBVH(true);

		return FImplicitObjectPtr(NewGeometry);
	}

	DECLARE_CYCLE_STAT(TEXT("FClusterUnionManager::ClaimNextUnionIndex"), STAT_ClaimNextUnionIndex, STATGROUP_Chaos);
	FClusterUnionIndex FClusterUnionManager::ClaimNextUnionIndex()
	{
		SCOPE_CYCLE_COUNTER(STAT_ClaimNextUnionIndex);
		if (ReusableIndices.IsEmpty())
		{
			return NextAvailableUnionIndex++;
		}
		else
		{
			return ReusableIndices.Pop(EAllowShrinking::No);
		}
	}

	DECLARE_CYCLE_STAT(TEXT("FClusterUnionManager::AddPendingExplicitIndexOperation"), STAT_AddPendingExplicitIndexOperation, STATGROUP_Chaos);
	void FClusterUnionManager::AddPendingExplicitIndexOperation(FClusterUnionExplicitIndex Index, EClusterUnionOperation Op, const TArray<FPBDRigidParticleHandle*>& Particles)
	{
		check(Op != EClusterUnionOperation::UpdateChildToParent);
		SCOPE_CYCLE_COUNTER(STAT_AddPendingExplicitIndexOperation);
		AddPendingOperation(PendingExplicitIndexOperations, Index, Op, Particles);
	}

	DECLARE_CYCLE_STAT(TEXT("FClusterUnionManager::AddPendingClusterIndexOperation"), STAT_AddPendingClusterIndexOperation, STATGROUP_Chaos);
	void FClusterUnionManager::AddPendingClusterIndexOperation(FClusterUnionIndex Index, EClusterUnionOperation Op, const TArray<FPBDRigidParticleHandle*>& Particles)
	{
		check(Op != EClusterUnionOperation::UpdateChildToParent);
		SCOPE_CYCLE_COUNTER(STAT_AddPendingClusterIndexOperation);
		AddPendingOperation(PendingClusterIndexOperations, Index, Op, Particles);
	}

	DECLARE_CYCLE_STAT(TEXT("FClusterUnionManager::FlushPendingOperations"), STAT_FlushPendingOperations, STATGROUP_Chaos);
	void FClusterUnionManager::FlushPendingOperations()
	{
		SCOPE_CYCLE_COUNTER(STAT_FlushPendingOperations);
		if (!PendingExplicitIndexOperations.IsEmpty())
		{
			// Go through every explicit index operation and convert them into a normal cluster index operation.
			// This could be made more efficient but shouldn't happen enough for it to really matter.
			for (const TPair<FClusterUnionExplicitIndex, FClusterOpMap>& OpMap : PendingExplicitIndexOperations)
			{
				const FClusterUnionIndex UnionIndex = GetOrCreateClusterUnionIndexFromExplicitIndex(OpMap.Key);
				for (const TPair<EClusterUnionOperation, TArray<FPBDRigidParticleHandle*>>& Op : OpMap.Value)
				{
					AddPendingClusterIndexOperation(UnionIndex, Op.Key, Op.Value);
				}
			}
			PendingExplicitIndexOperations.Empty();
		}

		if (!PendingClusterIndexOperations.IsEmpty())
		{
			for (TPair<FClusterUnionIndex, FClusterOpMap>& OpMap : PendingClusterIndexOperations)
			{
				// Is this sort necessary? Better to be safe than sorry. Since we need to guarantee that the UpdateChildToParent happens after add.
				OpMap.Value.KeyStableSort(
					[](EClusterUnionOperation A, EClusterUnionOperation B)
					{
						// remove must be bubbled up to the top (for the case when remove and add are used at the same time)
						if (bFixRemoveDuringSort)
						{
							if (A == EClusterUnionOperation::Remove)
							{
								return true;
							}
							else if (B == EClusterUnionOperation::Remove)
							{
								return false;
							}
						}

						return static_cast<int32>(A) < static_cast<int32>(B);
					}
				);


				for (TPair<EClusterUnionOperation, TArray<FPBDRigidParticleHandle*>>& Op : OpMap.Value)
				{
					switch (Op.Key)
					{
					case EClusterUnionOperation::Add:
					case EClusterUnionOperation::AddReleased:
						HandleAddOperation(OpMap.Key, Op.Value, Op.Key == EClusterUnionOperation::AddReleased);
						break;
					case EClusterUnionOperation::Remove:
						HandleRemoveOperation(OpMap.Key, Op.Value, EClusterUnionOperationTiming::Defer);
						break;
					case EClusterUnionOperation::UpdateChildToParent:
						HandleUpdateChildToParentOperation(OpMap.Key, Op.Value);
						break;
					}
				}
			}
			PendingClusterIndexOperations.Empty();

			if (FRewindData* RewindData = MEvolution.GetRewindData())
			{
				// Temp, since rewind doesn't handle altered cluster unions we tell it to never rewind past this frame for now, only relevant when resimulation is used.
				RewindData->BlockResim();
			}
		}

		HandleDeferredClusterUnionUpdateProperties();
	}

	FClusterUnion* FClusterUnionManager::FindClusterUnionFromExplicitIndex(FClusterUnionExplicitIndex Index)
	{
		FClusterUnionIndex* ClusterIndex = ExplicitIndexMap.Find(Index);
		if (ClusterIndex == nullptr)
		{
			return nullptr;
		}

		return FindClusterUnion(*ClusterIndex);
	}

	FClusterUnion* FClusterUnionManager::FindClusterUnion(FClusterUnionIndex Index)
	{
		return ClusterUnions.Find(Index);
	}

	const FClusterUnion* FClusterUnionManager::FindClusterUnion(FClusterUnionIndex Index) const
	{
		return ClusterUnions.Find(Index);
	}

	FClusterUnion* FClusterUnionManager::FindClusterUnionFromParticle(FPBDRigidParticleHandle* Particle)
	{
		FClusterUnionIndex ClusterUnionIndex = FindClusterUnionIndexFromParticle(Particle);
		return FindClusterUnion(ClusterUnionIndex);
	}

	const FClusterUnion* FClusterUnionManager::FindClusterUnionFromParticle(const FPBDRigidParticleHandle* Particle) const
	{
		FClusterUnionIndex ClusterUnionIndex = FindClusterUnionIndexFromParticle(Particle);
		return FindClusterUnion(ClusterUnionIndex);
	}

	DECLARE_CYCLE_STAT(TEXT("FClusterUnionManager::HandleAddOperation"), STAT_HandleAddOperation, STATGROUP_Chaos);
	void FClusterUnionManager::HandleAddOperation(FClusterUnionIndex ClusterIndex, const TArray<FPBDRigidParticleHandle*>& InParticles, bool bReleaseClustersFirst)
	{
		SCOPE_CYCLE_COUNTER(STAT_HandleAddOperation);
		FClusterUnion* Cluster = ClusterUnions.Find(ClusterIndex);
		if (!Cluster)
		{
			return;
		}

		// This ensures us that we only try to add particles that aren't already in the cluster already.
		// TODO: There's probably a better way to do this without having to reallocate another array - but these arrays should be small.
		TArray<FPBDRigidParticleHandle*> Particles = InParticles.FilterByPredicate(
			[this, Cluster](FPBDRigidParticleHandle* P)
			{
				if (!P)
				{
					return false;
				}

				if (FPBDRigidClusteredParticleHandle* ClusteredP = P->CastToClustered())
				{
					return Cluster->InternalCluster != ClusteredP->Parent();
				}

				return false;
			}
		);

		// If we're adding particles to a cluster we need to first make sure they're not part of any other cluster.
		// Book-keeping might get a bit odd if we try to add a particle to a new cluster and then only later remove the particle from its old cluster.
		HandleRemoveOperationWithClusterLookup(Particles, EClusterUnionOperationTiming::Defer);

		TGuardValue_Bitfield_Cleanup<TFunction<void()>> Cleanup(
			[this, OldGenerateClusterBreaking=MClustering.GetDoGenerateBreakingData()]() {
				MClustering.SetGenerateClusterBreaking(OldGenerateClusterBreaking);
			}
		);
		MClustering.SetGenerateClusterBreaking(false);

		// If a physics proxy was set already on the cluster we want to make sure that doesn't change.
		// This is needed to eventually be able to introduce a new physics proxy that gets attached to the
		// cluster union particle so that it can communicate with the game thread.
		IPhysicsProxyBase* OldProxy = Cluster->InternalCluster->PhysicsProxy();
		EPhysicsProxyType OldProxyType = OldProxy ? OldProxy->GetType() : EPhysicsProxyType::NoneType;

		// If we're a new cluster, we need to determine whether to start the cluster in a sleeping or dynamic state.
		// Only stay sleeping if all the particles we add are also sleeping.
		const bool bIsNewCluster = Cluster->ChildParticles.IsEmpty();
		bool bIsSleeping = true;
		bool bIsAnchored = false;

		// Use a set to guarantee that there's no duplicates. This is necessary since further removal operations assume that the particle is only in here once.
		const TSet<FPBDRigidParticleHandle*> ParticleSet{ Particles };

		TArray<FPBDRigidParticleHandle*> FinalParticlesToAdd;
		FinalParticlesToAdd.Reserve(ParticleSet.Num());

		// This is only relevant when bReleaseClustersFirst=true. This is used to be able to
		// properly notify the parent cluster about its child proxies.
		TMap<FPBDRigidParticleHandle*, FPBDRigidParticleHandle*> ChildToParentMap;

		
		for (FPBDRigidParticleHandle* Handle : ParticleSet)
		{
			if (!Handle)
			{
				continue;
			}

			if (FPBDRigidClusteredParticleHandle* ClusterHandle = Handle->CastToClustered())
			{
				// If this particle is a broken cluster without any children, then we should not be adding it to a ClusterUnion.
				// This can happen on the client if we receive an AddComponentToCluster on the same frame when we receive a GC repdata
				// that breaks all the particles and releases its children. Unfortunately the Add operation has already been queued
				if (bChaosClusterUnionDoNotAddEmptyClusters)
				{
					const bool bIsClusterThatHadChildren = (ClusterHandle->GetParticleType() == EParticleType::Clustered);
					if (bIsClusterThatHadChildren && (ClusterHandle->Parent() == nullptr) && ClusterHandle->Disabled() && (ClusterHandle->ClusterIds().NumChildren == 0))
					{
						UE_LOGF(LogChaos, Verbose, "FClusterUnionManager::HandleAddOperation rejecting particle because it is 'broken' %ls", *ClusterHandle->GetDebugName());
						continue;
					}
				}

				if (bReleaseClustersFirst)
				{
					TSet<FPBDRigidParticleHandle*> Children = MClustering.ReleaseClusterParticles(ClusterHandle, true);
					FinalParticlesToAdd.Append(Children.Array());

					for (FPBDRigidParticleHandle* Child : Children)
					{
						ChildToParentMap.Add(Child, ClusterHandle);
					}
				}
				else
				{
					FinalParticlesToAdd.Add(Handle);
				}

				bIsAnchored |= ClusterHandle->IsAnchored();
			}
			else
			{
				FinalParticlesToAdd.Add(Handle);
			}

			bIsSleeping &= Handle->ObjectState() == EObjectStateType::Sleeping;
		}

		if (FinalParticlesToAdd.IsEmpty())
		{
			return;
		}

		Cluster->ChildParticles.Append(FinalParticlesToAdd);

		// We need to call AddParticlesToCluster before we set the temporary ChildToParent. RemoveFromParent (which AddParticlesToCluster calls)
		// will set the particles' X/R based on ChildToParent if the particle has an old parent. This ChildToParent needs to be correct relative to the old parent,
		// not the new parent when AddParticlesToCluster is called.
		MClustering.AddParticlesToCluster(Cluster->InternalCluster, FinalParticlesToAdd, ChildToParentMap);

		// Cluster has one-way interaction only if all children are also one-way
		bool bIsOneWayInteraction = (bIsNewCluster) || Cluster->InternalCluster->OneWayInteraction();

		// Cluster uses MACD is any children require MACD (or it is already enabled on this cluster)
		bool bIsMACD = Cluster->InternalCluster->MACDEnabled();

		Private::FIterationSettings SolverIterationSettings = Cluster->InternalCluster->IterationSettings();


		// Use the minimum sleep multiplier of all member particles
		FRealSingle MinSleepThresholdMultiplier = TNumericLimits<FRealSingle>::Max();
		
		Chaos::FReal MaxLinearEtherDrag = Cluster->InternalCluster->LinearEtherDrag();
		Chaos::FReal MaxAngularEtherDrag = Cluster->InternalCluster->AngularEtherDrag();
		bool bEnableGravity = Cluster->InternalCluster->GravityEnabled();

		for (FPBDRigidParticleHandle* Particle : FinalParticlesToAdd)
		{
			if (Particle->GetGeometry() == nullptr)
			{
				FGeometryCollectionPhysicsProxy* GCPhysicsProxy = GetGCProxy(Particle);
				if (GCPhysicsProxy)
				{
					GCPhysicsProxy->CreateChildrenGeometry_Internal();
				}
			}

			if (!bIsNewCluster)
			{
				Cluster->PendingConnectivityOperations.Add({ Particle, EClusterUnionConnectivityOperation::Add });
				Cluster->AddPendingGeometryOperation(EClusterUnionGeometryOperation::Add, Particle);
			}

			bIsOneWayInteraction &= Particle->OneWayInteraction();
			bIsMACD |= Particle->MACDEnabled();
			MinSleepThresholdMultiplier = FMath::Min(MinSleepThresholdMultiplier, Particle->SleepThresholdMultiplier());
			MaxLinearEtherDrag = FMath::Max(MaxLinearEtherDrag, Particle->LinearEtherDrag());
			MaxAngularEtherDrag = FMath::Max(MaxAngularEtherDrag, Particle->AngularEtherDrag());
			bEnableGravity &= Particle->GravityEnabled(); // favor disabled gravity 

			SolverIterationSettings = Private::FIterationSettings::Merge(SolverIterationSettings, Particle->IterationSettings());

			if (!Cluster->ChildProperties.Contains(Particle))
			{
				FClusterUnionParticleProperties Properties;
				Properties.bIsAuxiliaryParticle = false;
				Cluster->ChildProperties.Add(Particle, Properties);
			}

			// Need to set a temporary ChildToParent on the particle. If this ChildToParent doesn't exist, and the particle goes through the removal process BEFORE
			// the deferred cluster union update properties is called, the position of the particle will get reset to the position of the cluster union.
			if (FPBDRigidClusteredParticleHandle* ClusterParticle = Particle->CastToClustered())
			{
				const FRigidTransform3 ClusterWorldTM(Cluster->InternalCluster->GetX(), Cluster->InternalCluster->GetR());
				const FRigidTransform3 Frame = GetParticleRigidFrameInClusterUnion(Particle, ClusterWorldTM);
				ClusterParticle->SetChildToParent(Frame);
			}

			MEvolution.GetParticles().MarkTransientDirtyParticle(Particle);
		}

		// For all the particles that have been added to the cluster we need to set their parent proxy to the
		// cluster's proxy if it exists. We need the proxy type check because for non-cluster union proxy backed unions,
		// the cluster union's proxy will be the proxy of the most recently added particle.
		if (OldProxy && OldProxyType == EPhysicsProxyType::ClusterUnionProxy)
		{
			for (FPBDRigidParticleHandle* Particle : FinalParticlesToAdd)
			{
				if (Particle && Particle->PhysicsProxy())
				{
					Particle->PhysicsProxy()->SetParentProxy(OldProxy);
				}
			}
		}

		if (bIsNewCluster && bIsAnchored && !Cluster->bAnchorLock)
		{
			// The anchored flag is taken care of in UpdateKinematicProperties so it must be set before that is called.
			Cluster->InternalCluster->SetIsAnchored(true);
		}

		constexpr EUpdateClusterUnionPropertiesFlags DefaultIncrementalFlags = EUpdateClusterUnionPropertiesFlags::IncrementalGenerateConnectionGraph
			| EUpdateClusterUnionPropertiesFlags::IncrementalGenerateGeometry;
		RequestDeferredClusterPropertiesUpdate(ClusterIndex, bIsNewCluster ? EUpdateClusterUnionPropertiesFlags::All : DefaultIncrementalFlags);

		if (OldProxy)
		{
			Cluster->InternalCluster->SetPhysicsProxy(OldProxy);
		}

		if (bIsNewCluster)
		{
			if (bIsSleeping)
			{
				MEvolution.SetParticleObjectState(Cluster->InternalCluster, Chaos::EObjectStateType::Sleeping);
			}

			if (!FinalParticlesToAdd.IsEmpty())
			{
				FPBDRigidParticleHandle* ParticlePropertySource = FinalParticlesToAdd[0];

				if (!Cluster->IsGravityOverrideSet())
				{
					Cluster->InternalCluster->SetGravityGroupIndex(ParticlePropertySource->GravityGroupIndex());
				}

				// Should be min or max of all children or something else that doesn't depend on order? 
				Cluster->InternalCluster->SetInitialOverlapDepenetrationVelocity(ParticlePropertySource->InitialOverlapDepenetrationVelocity());
			}
		}
		Cluster->InternalCluster->SetLinearEtherDrag(MaxLinearEtherDrag);
		Cluster->InternalCluster->SetAngularEtherDrag(MaxAngularEtherDrag);
		Cluster->InternalCluster->SetGravityEnabled(bEnableGravity);

		Cluster->InternalCluster->SetOneWayInteraction(bIsOneWayInteraction);
		Cluster->InternalCluster->SetMACDEnabled(bIsMACD);
		Cluster->InternalCluster->SetIterationSettings(SolverIterationSettings);

		if (MinSleepThresholdMultiplier != TNumericLimits<FRealSingle>::Max())
		{
			Cluster->InternalCluster->SetSleepThresholdMultiplier(MinSleepThresholdMultiplier);
		}
	}

	DECLARE_CYCLE_STAT(TEXT("FClusterUnionManager::DeferredClusterUnionUpdate"), STAT_DeferredClusterUnionUpdate, STATGROUP_Chaos);
	void FClusterUnionManager::DeferredClusterUnionUpdate(FClusterUnion& Union, EUpdateClusterUnionPropertiesFlags Flags)
	{
		SCOPE_CYCLE_COUNTER(STAT_DeferredClusterUnionUpdate);

		UpdateAllClusterUnionProperties(Union, Flags);

		if (!Union.ChildParticles.IsEmpty())
		{
			if (Union.InternalCluster->Disabled())
			{
				MEvolution.EnableParticle(Union.InternalCluster);
			}
			MEvolution.DirtyParticle(*Union.InternalCluster);
		}
		else
		{
			// Note that if we have 0 child particles, our implicit object union will have an invalid bounding box.
			// We must eject from the acceleration structure otherwise we risk cashes.
			MEvolution.DisableParticle(Union.InternalCluster);
		}
		MEvolution.GetParticles().MarkTransientDirtyParticle(Union.InternalCluster);
	}

	DECLARE_CYCLE_STAT(TEXT("FClusterUnionManager::HandleDeferredClusterUnionUpdateProperties"), STAT_HandleDeferredClusterUnionUpdateProperties, STATGROUP_Chaos);
	void FClusterUnionManager::HandleDeferredClusterUnionUpdateProperties()
	{
		SCOPE_CYCLE_COUNTER(STAT_HandleDeferredClusterUnionUpdateProperties);
		if (DeferredClusterUnionsForUpdateProperties.IsEmpty())
		{
			return;
		}

		for (const TPair<FClusterUnionIndex, EUpdateClusterUnionPropertiesFlags>& Kvp : DeferredClusterUnionsForUpdateProperties)
		{
			if (FClusterUnion* Union = FindClusterUnion(Kvp.Key))
			{
				DeferredClusterUnionUpdate(*Union, Kvp.Value);
			}
		}

		// Pending particles to undo child to parent lock is global across all cluster unions hence why it has to be done after.
		for (FPBDRigidClusteredParticleHandle* ChildParticle : PendingParticlesToUndoChildToParentLock)
		{
			if (ChildParticle)
			{
				ChildParticle->SetChildToParentLocked(false);
			}
		}
		PendingParticlesToUndoChildToParentLock.Empty();

		DeferredClusterUnionsForUpdateProperties.Reset();
	}

	DECLARE_CYCLE_STAT(TEXT("FClusterUnionManager::HandleRemoveOperation"), STAT_HandleRemoveOperation, STATGROUP_Chaos);
	void FClusterUnionManager::HandleRemoveOperation(FClusterUnionIndex ClusterIndex, const TArray<FPBDRigidParticleHandle*>& Particles, EClusterUnionOperationTiming UpdateClusterPropertiesTiming)
	{
		SCOPE_CYCLE_COUNTER(STAT_HandleRemoveOperation);
		FClusterUnion* Cluster = ClusterUnions.Find(ClusterIndex);
		if (!Cluster || Particles.IsEmpty())
		{
			return;
		}

		TGuardValue_Bitfield_Cleanup<TFunction<void()>> Cleanup(
			[this, OldGenerateClusterBreaking = MClustering.GetDoGenerateBreakingData()]() {
				MClustering.SetGenerateClusterBreaking(OldGenerateClusterBreaking);
			}
		);
		MClustering.SetGenerateClusterBreaking(false);

		// Use a set to guarantee that there's no duplicates. This is necessary since further removal operations assume that the particle is only in here once.
		const TSet<FPBDRigidParticleHandle*> ParticleSet{ Particles };

		IPhysicsProxyBase* OldProxy = Cluster->InternalCluster->PhysicsProxy();
		TArray<int32> ParticleIndicesToRemove;
		ParticleIndicesToRemove.Reserve(ParticleSet.Num());

		for (FPBDRigidParticleHandle* Handle : ParticleSet)
		{
			const int32 ParticleIndex = Cluster->ChildParticles.Find(Handle);
			if (ParticleIndex != INDEX_NONE)
			{
				ParticleIndicesToRemove.Add(ParticleIndex);

				// Remove the child to parent lock if it exists since it's no longer managed by the cluster union manager.
				if (FPBDRigidClusteredParticleHandle* ClusterHandle = Handle->CastToClustered())
				{
					ClusterHandle->SetChildToParentLocked(false);
				}

				// Remove the parent proxy only if it's a cluster union proxy.
				if (IPhysicsProxyBase* Proxy = Handle->PhysicsProxy(); Proxy && Proxy->GetParentProxy() && Proxy->GetParentProxy()->GetType() == EPhysicsProxyType::ClusterUnionProxy)
				{
					Proxy->SetParentProxy(nullptr);
				}

				Cluster->PendingConnectivityOperations.Add({ Handle, EClusterUnionConnectivityOperation::Remove });
				Cluster->AddPendingGeometryOperation(EClusterUnionGeometryOperation::Remove, Handle);

				MEvolution.GetParticles().MarkTransientDirtyParticle(Handle);
			}
		}

		// If we don't actually make a change to the cluster union then we don't actually want to trigger any further changes.
		if (ParticleIndicesToRemove.IsEmpty())
		{
			return;
		}

		ParticleIndicesToRemove.Sort();

		TArray<FPBDRigidParticleHandle*> NonMainReleasedParticles;
		NonMainReleasedParticles.Reserve(ParticleIndicesToRemove.Num());

		for (int32 Index = ParticleIndicesToRemove.Num() - 1; Index >= 0; --Index)
		{
			const int32 ParticleIndex = ParticleIndicesToRemove[Index];
			FPBDRigidParticleHandle* Particle = Cluster->ChildParticles[ParticleIndex];
			if (const FClusterUnionParticleProperties* Properties = Cluster->ChildProperties.Find(Particle))
			{
				if (Properties->bIsAuxiliaryParticle)
				{
					NonMainReleasedParticles.Add(Particle);
				}
				Cluster->ChildProperties.Remove(Particle);
			}

			// This can't be RemoveAtSwap otherwise there will be a mismatch between the index of a particle
			// and the index of its corresponding shape in the cluster union's shape array. There is currently
			// an assumption that the two will always match each other.
			Cluster->ChildParticles.RemoveAt(ParticleIndex);

			if (FPBDRigidClusteredParticleHandle* ClusterParticle = Particle->CastToClustered())
			{
				// If we remove the particle we don't need to update its child to parent any longer.
				if (FClusterUnionChildToParentUpdate* Update = PendingChildToParentUpdates.Find(ClusterParticle))
				{
					// Doing this here ensures that this particle is still part of this cluster union and hasn't yet been added into another cluster union.
					if (Update->ClusterUnionIndex == ClusterIndex)
					{
						PendingChildToParentUpdates.Remove(ClusterParticle);
					}
				}

				// TODO: We probably won't ever run into a situation where this actually does anything since
				// this container gets cleared pretty shortly after things are added to it. But to be safe
				// we probably need some sort of verification that we're removing a particle that isn't associated
				// with some other cluster union.
				PendingParticlesToUndoChildToParentLock.Remove(ClusterParticle);
			}
		}

		MClustering.RemoveParticlesFromCluster(Cluster->InternalCluster, ParticleSet.Array());
		if (MClustering.ShouldThrottleParticleRelease())
		{
			// Only want to throttle the release of non-main particles.
			MClustering.ThrottleReleasedParticlesIfNecessary(NonMainReleasedParticles);
		}

		// Removing a particle should have no bearing on the proxy of the cluster.
		// This gets changed because we go through an internal initialization route when we update the cluster union particle's properties.
		Cluster->InternalCluster->SetPhysicsProxy(OldProxy);

		constexpr EUpdateClusterUnionPropertiesFlags RemoveUpdateFlags = EUpdateClusterUnionPropertiesFlags::IncrementalGenerateConnectionGraph
			| EUpdateClusterUnionPropertiesFlags::UpdateKinematicProperties
			| EUpdateClusterUnionPropertiesFlags::IncrementalGenerateGeometry
			| EUpdateClusterUnionPropertiesFlags::ConnectivityCheck;
		switch (UpdateClusterPropertiesTiming)
		{
		case EClusterUnionOperationTiming::Immediate:
			DeferredClusterUnionUpdate(*Cluster, RemoveUpdateFlags);
			break;
		case EClusterUnionOperationTiming::Defer:
			RequestDeferredClusterPropertiesUpdate(ClusterIndex, RemoveUpdateFlags);
			break;
		}		
	}

	DECLARE_CYCLE_STAT(TEXT("FClusterUnionManager::HandleRemoveOperationWithClusterLookup"), STAT_HandleRemoveOperationWithClusterLookup, STATGROUP_Chaos);
	void FClusterUnionManager::HandleRemoveOperationWithClusterLookup(const TArray<FPBDRigidParticleHandle*>& InParticles, EClusterUnionOperationTiming UpdateClusterPropertiesTiming)
	{
		SCOPE_CYCLE_COUNTER(STAT_HandleRemoveOperationWithClusterLookup);
		TMap<FClusterUnionIndex, TSet<FPBDRigidParticleHandle*>> ParticlesPerCluster;
		for (FPBDRigidParticleHandle* Particle : InParticles)
		{
			if (const int32 Index = FindClusterUnionIndexFromParticle(Particle); Index != INDEX_NONE)
			{
				ParticlesPerCluster.FindOrAdd(Index).Add(Particle);
			}
		}

		for (const TPair<FClusterUnionIndex, TSet<FPBDRigidParticleHandle*>>& Kvp : ParticlesPerCluster)
		{
			HandleRemoveOperation(Kvp.Key, Kvp.Value.Array(), UpdateClusterPropertiesTiming);
		}
	}

	DECLARE_CYCLE_STAT(TEXT("FClusterUnionManager::UpdateClusterUnionProperties"), STAT_UpdateClusterUnionProperties, STATGROUP_Chaos);
	void FClusterUnionManager::UpdateAllClusterUnionProperties(FClusterUnion& ClusterUnion, EUpdateClusterUnionPropertiesFlags Flags)
	{
		SCOPE_CYCLE_COUNTER(STAT_UpdateClusterUnionProperties);
		// Update cluster properties.
		FMatrix33 ClusterInertia(0);

		const bool bRecomputeMassOrientation = EnumHasAnyFlags(Flags, EUpdateClusterUnionPropertiesFlags::RecomputeMassOrientation) && ClusterUnion.bNeedsXRInitialization;
		TSet<FPBDRigidParticleHandle*> FullChildrenSet(ClusterUnion.ChildParticles);

		UpdateClusterMassProperties(ClusterUnion.InternalCluster, FullChildrenSet);

		// Position the internal cluster at the CoM of the children
		if (bRecomputeMassOrientation)
		{
			MoveClusterToMassOffset(ClusterUnion.InternalCluster, EMassOffsetType::Position);
			ClusterUnion.bNeedsXRInitialization = false;
		}

		if (EnumHasAnyFlags(Flags, EUpdateClusterUnionPropertiesFlags::UpdateKinematicProperties))
		{
			if (ClusterUnion.bAnchorLock)
			{
				// Skip UpdateKinematicPropertiesso as we won't need to look at our children for this.
				if (ClusterUnion.InternalCluster->IsAnchored())
				{
					MEvolution.SetParticleObjectState(ClusterUnion.InternalCluster, EObjectStateType::Kinematic);
				}
				else
				{
					const EObjectStateType TransferedState = ClusterUnion.InternalCluster->IsSleeping() ? EObjectStateType::Sleeping : EObjectStateType::Dynamic;
					MEvolution.SetParticleObjectState(ClusterUnion.InternalCluster, TransferedState);
					MEvolution.SetParticleKinematicTarget(ClusterUnion.InternalCluster, FKinematicTarget());
				}
			}
			else
			{
				UpdateKinematicProperties(ClusterUnion.InternalCluster, MClustering.GetChildrenMap(), MEvolution);
			}
		}

		// We must reset collisions etc on this particle since geometry will be changed
		MEvolution.InvalidateParticle(ClusterUnion.InternalCluster);

		// The recreation of the geometry must happen after the call to UpdateClusterMassProperties.
		// Creating the geometry requires knowing the relative frame between the parent cluster and the child clusters. The
		// parent transform is not set properly for a new empty cluster until UpdateClusterMassProperties is called for the first time.
		if (EnumHasAnyFlags(Flags, EUpdateClusterUnionPropertiesFlags::ForceGenerateGeometry))
		{
			ForceRegenerateGeometry(ClusterUnion, FullChildrenSet);
		}
		else if (EnumHasAnyFlags(Flags, EUpdateClusterUnionPropertiesFlags::IncrementalGenerateGeometry))
		{
			FlushIncrementalGeometryOperations(ClusterUnion);
		}
		
		if (ClusterUnion.bGenerateConnectivityEdges)
		{
			if(EnumHasAnyFlags(Flags, EUpdateClusterUnionPropertiesFlags::ForceGenerateConnectionGraph))
			{
				GenerateConnectionGraph(MClustering, ClusterUnion);
				ClusterUnion.PendingConnectivityOperations.Empty();
			}
			else if (EnumHasAnyFlags(Flags, EUpdateClusterUnionPropertiesFlags::IncrementalGenerateConnectionGraph))
			{
				FlushIncrementalConnectivityGraphOperations(ClusterUnion);
			}

			if (EnumHasAnyFlags(Flags, EUpdateClusterUnionPropertiesFlags::ConnectivityCheck))
			{
				// TODO: Can probably argue that cluster union connectivity should be moved into the
				// cluster union manager instead?
				MClustering.HandleConnectivityOnReleaseClusterParticle(ClusterUnion.InternalCluster, false);
			}
		}
		// Build the convex optimizer if required
		MClustering.BuildConvexOptimizer(ClusterUnion.InternalCluster);
	}

	void FClusterUnionManager::AddParticleToConnectionGraphInCluster(FClusterUnion& ClusterUnion, FPBDRigidParticleHandle* Particle)
	{
		AddParticleToConnectionGraph(MClustering, ClusterUnion, Particle);
	}

	DECLARE_CYCLE_STAT(TEXT("FClusterUnionManager::GenerateInterclusterEdgesForParticle"), STAT_GenerateInterclusterEdgesForParticle, STATGROUP_Chaos);
	void FClusterUnionManager::GenerateInterclusterEdgesForParticle(FClusterUnion& ClusterUnion, FPBDRigidParticleHandle* Particle)
	{
		SCOPE_CYCLE_COUNTER(STAT_GenerateInterclusterEdgesForParticle);
		if (!Particle)
		{
			return;
		}

		// Just a safety check really to make sure the particle is actually in the cluster union.
		FClusterUnionParticleProperties* Properties = ClusterUnion.ChildProperties.Find(Particle);
		if (!Properties || Properties->bEdgesAreGenerated)
		{
			return;
		}

		auto PrepareParticleForInterclusterEdges = [](FPBDRigidParticleHandle& Particle)
		{
			IPhysicsProxyBase* Proxy = Particle.PhysicsProxy();
			if (!Proxy)
			{
				return;
			}

			if (Proxy->GetType() == FGeometryCollectionPhysicsProxy::ConcreteType())
			{
				FGeometryCollectionPhysicsProxy* GCProxy = static_cast<FGeometryCollectionPhysicsProxy*>(Proxy);
				check(GCProxy != nullptr);
				GCProxy->CreateChildrenGeometry_Internal();
			}
		};

		PrepareParticleForInterclusterEdges(*Particle);

		FPBDRigidClusteredParticleHandle* ClusteredParticle = Particle->CastToClustered();
		if (!ClusteredParticle)
		{
			return;
		}

		const TArray<Chaos::FConnectivityEdge>& Edges = ClusteredParticle->ConnectivityEdges();
		for (const Chaos::FConnectivityEdge& Edge : Edges)
		{
			if (Edge.Sibling != nullptr)
			{
				PrepareParticleForInterclusterEdges(*Edge.Sibling);
				GenerateInterclusterEdgesBetweenParticles(ClusterUnion, Particle, Edge.Sibling);
			}
		}

		Properties->bEdgesAreGenerated = true;
	}

	DECLARE_CYCLE_STAT(TEXT("FClusterUnionManager::GenerateInterclusterEdgesBetweenParticles"), STAT_GenerateInterclusterEdgesBetweenParticles, STATGROUP_Chaos);
	void FClusterUnionManager::GenerateInterclusterEdgesBetweenParticles(FClusterUnion& ClusterUnion, FPBDRigidParticleHandle* Particle, FPBDRigidParticleHandle* OtherParticle)
	{
		SCOPE_CYCLE_COUNTER(STAT_GenerateInterclusterEdgesBetweenParticles);
		if (!Particle || !OtherParticle)
		{
			return;
		}

		FClusterUnionParticleProperties* Properties = ClusterUnion.ChildProperties.Find(Particle);
		if (!Properties)
		{
			return;
		}

		if (bChaosClusterUnionGenerateInterclusterEdges && Particle && OtherParticle)
		{
			// Only generate intercluster edges for main particles. Auxiliary particles that are just bits and pieces of geometry collections
			// shouldn't also generate intercluster edges.
			if (bFixPartialDestruction || !Properties || !Properties->bIsAuxiliaryParticle)
			{
				const TArray<FPBDRigidParticleHandle*>& ParticleChildren = MClustering.GetChildrenMap().FindRef(Particle->CastToClustered());
				const TArray<FPBDRigidParticleHandle*>& OtherChildren = MClustering.GetChildrenMap().FindRef(OtherParticle->CastToClustered());

				TSet<FPBDRigidParticleHandle*> FromSet{ ParticleChildren };
				TSet<FPBDRigidParticleHandle*> ToSet{ OtherChildren };

				TArray<FPBDRigidParticleHandle*> AllParticles;
				AllParticles.Reserve(FromSet.Num() + ToSet.Num());
				AllParticles.Append(ParticleChildren);
				AllParticles.Append(OtherChildren);

				FClusterCreationParameters Parameters{ 0.3f, 100, false, false };
				Parameters.ConnectionMethod = FClusterCreationParameters::EConnectionMethod::BoundsOverlapFilteredDelaunayTriangulation;
				Parameters.ConnectionGraphBoundsFilteringMargin = 1.0;

				MClustering.GenerateConnectionGraph(AllParticles, Parameters, &FromSet, &ToSet);
			}
		}
	}

	DECLARE_CYCLE_STAT(TEXT("FClusterUnionManager::FlushIncrementalConnectivityGraphOperations"), STAT_FlushIncrementalConnectivityGraphOperations, STATGROUP_Chaos);
	void FClusterUnionManager::FlushIncrementalConnectivityGraphOperations(FClusterUnion& ClusterUnion)
	{
		SCOPE_CYCLE_COUNTER(STAT_FlushIncrementalConnectivityGraphOperations);
		for (const TPair<FPBDRigidParticleHandle*, EClusterUnionConnectivityOperation>& Op : ClusterUnion.PendingConnectivityOperations)
		{
			if (Op.Value == EClusterUnionConnectivityOperation::Add)
			{
				AddParticleToConnectionGraph(MClustering, ClusterUnion, Op.Key);
			}
			else if (Op.Value == EClusterUnionConnectivityOperation::Remove)
			{
				RemoveClusterUnionEdges(MClustering, ClusterUnion, Op.Key);
			}
		}

		ClusterUnion.PendingConnectivityOperations.Empty();
	}

	DECLARE_CYCLE_STAT(TEXT("FClusterUnionManager::ForceRegenerateGeometry"), STAT_ForceRegenerateGeometry, STATGROUP_Chaos);
	void FClusterUnionManager::ForceRegenerateGeometry(FClusterUnion& ClusterUnion, const TSet<FPBDRigidParticleHandle*>& FullChildrenSet)
	{
		SCOPE_CYCLE_COUNTER(STAT_ForceRegenerateGeometry);

		// Must destroy collisions before modifying geometry
		MEvolution.InvalidateParticle(ClusterUnion.InternalCluster);

		ClusterUnion.InternalCluster->SetGeometry(MakeImplicitObjectPtr<FImplicitObjectUnionClustered>());

		ModifyAdditionOfChildrenToClusterUnionGeometry(
			ClusterUnion.InternalCluster,
			ClusterUnion.ChildParticles,
			ClusterUnion.ClusterUnionParameters.ActorId,
			ClusterUnion.ClusterUnionParameters.ComponentId,
			[this, &ClusterUnion, &FullChildrenSet]()
			{
				ClusterUnion.Geometry = ForceRecreateClusterUnionGeometry(ClusterUnion);
				UpdateGeometry(ClusterUnion.InternalCluster, FullChildrenSet, MClustering.GetChildrenMap(), ClusterUnion.Geometry, ClusterUnion.Parameters);
			}
		);

		ClusterUnion.GeometryChildParticles = ClusterUnion.ChildParticles;
		ClusterUnion.ClearAllPendingGeometryOperations();
	}

	DECLARE_CYCLE_STAT(TEXT("FClusterUnionManager::FlushIncrementalGeometryOperations"), STAT_FlushIncrementalGeometryOperations, STATGROUP_Chaos);
	void FClusterUnionManager::FlushIncrementalGeometryOperations(FClusterUnion& ClusterUnion)
	{
		SCOPE_CYCLE_COUNTER(STAT_FlushIncrementalGeometryOperations);
		check(ClusterUnion.Geometry != nullptr);

		if (ClusterUnion.Geometry->template IsA<FImplicitObjectUnionClustered>())
		{
			ForceRegenerateGeometry(ClusterUnion, TSet<FPBDRigidParticleHandle*>{ClusterUnion.ChildParticles});
			return;
		}

		FImplicitObjectUnion* ImplicitUnion = ClusterUnion.Geometry->template AsA<FImplicitObjectUnion>();
		check(ImplicitUnion != nullptr);

		const TArray<FPBDRigidParticleHandle*>& PendingGeometryAdditions = ClusterUnion.GetPendingGeometryOperationParticles(EClusterUnionGeometryOperation::Add);
		const TArray<FPBDRigidParticleHandle*>& PendingGeometryRemovals = ClusterUnion.GetPendingGeometryOperationParticles(EClusterUnionGeometryOperation::Remove);
		const TArray<FPBDRigidParticleHandle*>& PendingGeometryRefresh = ClusterUnion.GetPendingGeometryOperationParticles(EClusterUnionGeometryOperation::Refresh);
		if (PendingGeometryAdditions.IsEmpty() && PendingGeometryRemovals.IsEmpty() && PendingGeometryRefresh.IsEmpty())
		{
			// NOTE: Early out is important to prevent collision reset and BVH rebuild when there are no changes
			return;
		}

		// We are about to change the geometry so clear all cached collisions and prevent the BVH from being rebuilt until we are done
		MEvolution.InvalidateParticle(ClusterUnion.InternalCluster);
		ImplicitUnion->SetAllowBVH(false);

		const FRigidTransform3 ClusterWorldTM(ClusterUnion.InternalCluster->GetX(), ClusterUnion.InternalCluster->GetR());
		if (!PendingGeometryAdditions.IsEmpty())
		{
			ModifyAdditionOfChildrenToClusterUnionGeometry(
				ClusterUnion.InternalCluster,
				PendingGeometryAdditions,
				ClusterUnion.ClusterUnionParameters.ActorId,
				ClusterUnion.ClusterUnionParameters.ComponentId,
				[this, &ClusterUnion, &PendingGeometryAdditions, &ClusterWorldTM]()
				{

					TArray<Chaos::FImplicitObjectPtr> Objects;
					Objects.Reserve(PendingGeometryAdditions.Num());
					ClusterUnion.GeometryChildParticles.Reserve(ClusterUnion.GeometryChildParticles.Num() + PendingGeometryAdditions.Num());

					for (FPBDRigidParticleHandle* Child : PendingGeometryAdditions)
					{
						const FRigidTransform3 Frame = GetParticleRigidFrameInClusterUnion(Child, ClusterWorldTM);
						if (Child->GetGeometry())
						{
							ClusterUnion.GeometryChildParticles.Add(Child);
							Objects.Add(Chaos::FImplicitObjectPtr(CreateTransformGeometryForClusterUnion<EThreadContext::Internal>(Child, Frame)));
						}
					}

					if (!Objects.IsEmpty())
					{
						ClusterUnion.InternalCluster->MergeGeometry(MoveTemp(Objects));
					}
				}
			);

			ClusterUnion.ClearPendingGeometryOperations(EClusterUnionGeometryOperation::Add);
		}

		if (!PendingGeometryRemovals.IsEmpty())
		{
			RemoveParticlesFromClusterUnionGeometry(ClusterUnion.InternalCluster, PendingGeometryRemovals, ClusterUnion.GeometryChildParticles);
			ClusterUnion.Geometry = ClusterUnion.InternalCluster->GetGeometry();
			ClusterUnion.ClearPendingGeometryOperations(EClusterUnionGeometryOperation::Remove);

			ImplicitUnion = ClusterUnion.Geometry->template AsA<FImplicitObjectUnion>();
			check(ImplicitUnion != nullptr);
		}

		check(ClusterUnion.Geometry != nullptr);
		// Need to double check the geometry here. Did we get switched to a FImplicitObjectUnionClustered?
		// In that case we need to skip the refresh.
		const bool bIsUnionClustered = ClusterUnion.Geometry->template IsA<FImplicitObjectUnionClustered>();

		if (!PendingGeometryRefresh.IsEmpty() && !bIsUnionClustered)
		{
			// For each particle we need to find the corresponding shape.
			// Note that by the time we get to handling PendimgGeometryRefresh, we can once again
			// make the assumption that the number of children particles = number of shapes.
			const FShapesArray& ShapesArray = ClusterUnion.InternalCluster->ShapesArray();
			check(ClusterUnion.ChildParticles.Num() == ShapesArray.Num());

			bool bAnyTransformChanged = false;
			for (FPBDRigidParticleHandle* Particle : PendingGeometryRefresh)
			{
				if (!Particle)
				{
					continue;
				}

				const int32 Index = ClusterUnion.ChildParticles.Find(Particle);
				if (Index == INDEX_NONE || !ShapesArray.IsValidIndex(Index))
				{
					continue;
				}

				const TUniquePtr<Chaos::FPerShapeData>& TemplateShape = Particle->ShapesArray()[0];
				if (!TemplateShape)
				{
					continue;
				}

				if (FImplicitObjectRef ImplicitGeometry = ShapesArray[Index]->GetGeometry())
				{
					if (FImplicitObjectTransformed* Transformed = ImplicitGeometry->AsA<FImplicitObjectTransformed>())
					{
						const FRigidTransform3 Frame = GetParticleRigidFrameInClusterUnion(Particle, ClusterWorldTM);
						if (!Transformed->GetTransform().Equals(Frame))
						{
							Transformed->SetTransform(Frame);
							bAnyTransformChanged = true;
						}
					}
				}

				// @todo(chaos): we should probably rebuild the bounds if the sim and query filters change as well
				TransferClusterUnionShapeData(
					ShapesArray[Index],
					Particle,
					TemplateShape,
					ClusterUnion.ClusterUnionParameters.ActorId,
					ClusterUnion.ClusterUnionParameters.ComponentId
				);
			}

			ClusterUnion.ClearPendingGeometryOperations(EClusterUnionGeometryOperation::Refresh);

			if (bAnyTransformChanged)
			{
				// We have changed the transforms of some children so we must update the bounds
				ClusterUnion.InternalCluster->UpdateWorldSpaceState(ClusterUnion.InternalCluster->GetTransformPQ(), FVec3(0));
			}
		}

		// Re-enable the BVH to rebuild it
		ImplicitUnion->SetAllowBVH(true);
	}

	DECLARE_CYCLE_STAT(TEXT("FClusterUnionManager::GetOrCreateClusterUnionIndexFromExplicitIndex"), STAT_GetOrCreateClusterUnionIndexFromExplicitIndex, STATGROUP_Chaos);
	FClusterUnionIndex FClusterUnionManager::GetOrCreateClusterUnionIndexFromExplicitIndex(FClusterUnionExplicitIndex InIndex)
	{
		SCOPE_CYCLE_COUNTER(STAT_GetOrCreateClusterUnionIndexFromExplicitIndex);
		FClusterUnionIndex* OutIndex = ExplicitIndexMap.Find(InIndex);
		if (OutIndex != nullptr)
		{
			return *OutIndex;
		}

		FClusterUnionCreationParameters OtherParams;
		OtherParams.ExplicitIndex = InIndex;

		FClusterUnionIndex NewIndex = CreateNewClusterUnion(DefaultClusterCreationParameters(), OtherParams);
		ExplicitIndexMap.Add(InIndex, NewIndex);
		return NewIndex;
	}

	FClusterCreationParameters FClusterUnionManager::DefaultClusterCreationParameters() const
	{
		 FClusterCreationParameters Parameters{ 0.3f, 100, false, FRigidClustering::ShouldUnionsHaveCollisionParticles() };
		 Parameters.ConnectionMethod = MClustering.GetClusterUnionConnectionType();
		 return Parameters;
	}

	DECLARE_CYCLE_STAT(TEXT("FClusterUnionManager::FindClusterUnionIndexFromParticle"), STAT_FindClusterUnionIndexFromParticle, STATGROUP_Chaos);
	FClusterUnionIndex FClusterUnionManager::FindClusterUnionIndexFromParticle(const FPBDRigidParticleHandle* ChildParticle) const
	{
		SCOPE_CYCLE_COUNTER(STAT_FindClusterUnionIndexFromParticle);
		if (!ChildParticle)
		{
			return INDEX_NONE;
		}

		if (const FPBDRigidClusteredParticleHandle* ChildClustered = ChildParticle->CastToClustered())
		{
			if (const FPBDRigidClusteredParticleHandle* Parent = ChildClustered->Parent())
			{
				// Recursion should be fine here since the hierarchy should be fairly shallow.
				return FindClusterUnionIndexFromParticle(Parent);
			}

			// The only other check to do here is to see if this is the cluster union particle itself.
			const int32 ClusterGroupIndex = ChildClustered->ClusterGroupIndex();
			if (const FClusterUnion* ClusterUnion = FindClusterUnion(FMath::Abs(ClusterGroupIndex)))
			{
				// This is a sanity check which may or may not be necessary.
				// TODO: Can probably be safely removed once we deprecate cluster group indices on GCs.
				if (ClusterUnion->InternalCluster == ChildClustered)
				{
					return ClusterUnion->InternalIndex;
				}
			}
		}
		
		return INDEX_NONE;
	}

	bool FClusterUnionManager::IsClusterUnionParticle(FPBDRigidClusteredParticleHandle* Particle)
	{
		FClusterUnionIndex UnionIndex = FindClusterUnionIndexFromParticle(Particle);
		if (FClusterUnion* Union = FindClusterUnion(UnionIndex))
		{
			return Union->InternalCluster == Particle;
		}
		return false;
	}

	DECLARE_CYCLE_STAT(TEXT("FClusterUnionManager::UpdateClusterUnionParticlesChildToParent"), STAT_UpdateClusterUnionParticlesChildToParent, STATGROUP_Chaos);
	void FClusterUnionManager::UpdateClusterUnionParticlesChildToParent(FClusterUnionIndex Index, const TArray<FPBDRigidParticleHandle*>& Particles, const TArray<FTransform>& ChildToParent, bool bLock)
	{
		SCOPE_CYCLE_COUNTER(STAT_UpdateClusterUnionParticlesChildToParent);
		AddPendingOperation(PendingClusterIndexOperations, Index, EClusterUnionOperation::UpdateChildToParent, Particles);

		for (int32 InputIndex = 0; InputIndex < Particles.Num() && InputIndex < ChildToParent.Num(); ++InputIndex)
		{
			if (!Particles[InputIndex])
			{
				continue;
			}

			if (FPBDRigidClusteredParticleHandle* ClusterParticle = Particles[InputIndex]->CastToClustered())
			{
				FClusterUnionChildToParentUpdate& Update = PendingChildToParentUpdates.FindOrAdd(ClusterParticle);
				// If the current existing update wants to lock and we're not also locking, we can discard this new update.
				if (Update.bLock && !bLock)
				{
					continue;
				}
				Update.ChildToParent = ChildToParent[InputIndex];
				Update.bLock = bLock;
				Update.ClusterUnionIndex = Index;
			}
		}
	}

	DECLARE_CYCLE_STAT(TEXT("FClusterUnionManager::HandleUpdateChildToParentOperation"), STAT_HandleUpdateChildToParentOperation, STATGROUP_Chaos);
	void FClusterUnionManager::HandleUpdateChildToParentOperation(FClusterUnionIndex ClusterIndex, const TArray<FPBDRigidParticleHandle*>& Particles)
	{
		SCOPE_CYCLE_COUNTER(STAT_UpdateClusterUnionParticlesChildToParent);
		FClusterUnion* ClusterUnion = ClusterUnions.Find(ClusterIndex);
		if (!ClusterUnion || Particles.IsEmpty())
		{
			return;
		}

		// We need to keep track of all the particles we touched here. We need this because UpdateClusterUnionParticlesChildToParent
		// is an *authoritative* update on the ChildToParent of the particle. However, UpdateAllClusterUnionProperties in certain cases
		// may try to recompute the ChildToParent using the position of the particle. To counteract this, we will force the ChildToParent
		// to be *temporarily* locked for the duration of the *next* UpdateAllClusterUnionProperties. However, unless bLock is true, we will restore the
		// lock state of the particle to what it was previously.

		bool bMadeChanges = false;
		for (FPBDRigidParticleHandle* Particle : Particles)
		{
			if (!Particle)
			{
				return;
			}

			FPBDRigidParticleHandle* RigidParticle = Particle->CastToRigidParticle();
			FPBDRigidClusteredParticleHandle* ClusteredParticle = Particle->CastToClustered();

			const int32 ChildIndex = ClusterUnion->ChildParticles.Find(RigidParticle);
			if (ChildIndex != INDEX_NONE && ClusterUnion->InternalCluster)
			{
				if (ClusteredParticle)
				{
					if (const FClusterUnionChildToParentUpdate* Update = PendingChildToParentUpdates.Find(ClusteredParticle))
					{
						if (Update->ClusterUnionIndex != ClusterIndex)
						{
							continue;
						}

						const FRigidTransform3 ChildToParent = Update->ChildToParent;
						ClusteredParticle->SetChildToParent(ChildToParent);

						if (!Update->bLock && !ClusteredParticle->IsChildToParentLocked())
						{
							PendingParticlesToUndoChildToParentLock.Add(ClusteredParticle);
						}
						ClusteredParticle->SetChildToParentLocked(true);

						// Update the child's world transform to be consistent with its ChildToParent transform
						const FPBDRigidClusteredParticleHandle* ParentHandle = ClusterUnion->InternalCluster;
						const FRigidTransform3 ParticleToWorld = ChildToParent * FRigidTransform3(ParentHandle->GetX(), ParentHandle->GetR());
						ClusteredParticle->SetX(ParticleToWorld.GetTranslation());
						ClusteredParticle->SetP(ParticleToWorld.GetTranslation());
						ClusteredParticle->SetR(ParticleToWorld.GetRotation());
						ClusteredParticle->SetQ(ParticleToWorld.GetRotation());

						// We need to mark the child handle to be dirty so that its proxy gets a chance to sync back to the GC 
						MEvolution.GetParticles().MarkTransientDirtyParticle(ClusteredParticle);

						// A child to parent update needs to remove *and* add to the connectivity graph (in that order) since
						// the child to parent update might move the node so far away as to make the old connectivity edges incorrect.
						ClusterUnion->PendingConnectivityOperations.Add({ Particle, EClusterUnionConnectivityOperation::Remove });
						ClusterUnion->PendingConnectivityOperations.Add({ Particle, EClusterUnionConnectivityOperation::Add });

						// A child to parent update also requires the geometry to be refreshed since its transform is changed.
						ClusterUnion->AddPendingGeometryOperation(EClusterUnionGeometryOperation::Refresh, Particle);

						bMadeChanges = true;
					}
				}
			}

			PendingChildToParentUpdates.Remove(ClusteredParticle);
		}

		if (bMadeChanges)
		{
			MEvolution.GetParticles().MarkTransientDirtyParticle(ClusterUnion->InternalCluster, false);

			constexpr EUpdateClusterUnionPropertiesFlags Flags = EUpdateClusterUnionPropertiesFlags::IncrementalGenerateConnectionGraph
				| EUpdateClusterUnionPropertiesFlags::IncrementalGenerateGeometry;
			RequestDeferredClusterPropertiesUpdate(ClusterIndex, Flags);
		}
	}

	bool FClusterUnionManager::IsDirectlyConnectedToMainParticleInClusterUnion(const FClusterUnion& ClusterUnion, FPBDRigidParticleHandle* Particle) const
	{
		auto IsParticleMainParticle = [&ClusterUnion](const FPBDRigidClusteredParticleHandle* ClusterParticle)
		{
			if (!ClusterParticle)
			{
				return false;
			}

			if (const FClusterUnionParticleProperties* Props = ClusterUnion.ChildProperties.Find(ClusterParticle))
			{
				return !Props->bIsAuxiliaryParticle;
			}

			return false;
		};

		if (FPBDRigidClusteredParticleHandle* ClusterParticle = Particle->CastToClustered())
		{
			if (IsParticleMainParticle(ClusterParticle))
			{
				return true;
			}

			for (const FConnectivityEdge& Edge : ClusterParticle->ConnectivityEdges())
			{
				if (Edge.Sibling && IsInterclusterEdge(*ClusterParticle, Edge))
				{
					// We don't want to check the sibling, we want to check the parent of the sibling that's actually in the cluster union
					if (IsParticleMainParticle(GetParentParticleInClusterUnion(ClusterUnion, Edge.Sibling->CastToClustered())))
					{
						return true;
					}
				}
			}
		}

		return false;
	}

	void FClusterUnionManager::RequestDeferredClusterPropertiesUpdate(FClusterUnionIndex ClusterIndex, EUpdateClusterUnionPropertiesFlags Flags)
	{
		EUpdateClusterUnionPropertiesFlags& ExistingFlags = DeferredClusterUnionsForUpdateProperties.FindOrAdd(ClusterIndex);
		EnumAddFlags(ExistingFlags, Flags);
	}
}