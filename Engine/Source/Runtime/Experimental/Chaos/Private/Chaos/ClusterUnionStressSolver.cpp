// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/ClusterUnionStressSolver.h"
#include "Chaos/ClusterUnionManager.h"

#include "Chaos/DebugDrawQueue.h"

namespace
{
	bool bClusterUnionStressSolverEnableDebugDraw = false;
	FAutoConsoleVariableRef CVarClusterUnionStressSolverEnableDebugDraw(TEXT("ClusterUnion.StressSolver.EnableDebugDraw"), bClusterUnionStressSolverEnableDebugDraw, TEXT("When enabled, this will draw visual debug information for about the stress solver execution."));

	float ClusterUnionStressSolverStrengthScalar = 1.f;
	FAutoConsoleVariableRef CVarClusterUnionStressSolverStrengthScalar(TEXT("ClusterUnion.StressSolver.StrengthScalar"), ClusterUnionStressSolverStrengthScalar, TEXT("Materioal strength scalar ( <1: weaker, >1: stronger)"));

	float ComputeAreaFromBoundingBoxOverlap(const Chaos::FAABB3& BoxA, const Chaos::FAABB3& BoxB)
	{
		// if the two box don't overlap, we'll get an inside out box 
		// but we are still using it to compute the area as an approximation
		const Chaos::FAABB3 OverlapBox = BoxA.GetIntersection(BoxB);

		const Chaos::FVec3 Extents = OverlapBox.Extents();
		const Chaos::FVec3 CenterToCenterNormal = (BoxA.GetCenter() - BoxB.GetCenter()).GetSafeNormal();

		const Chaos::FVec3 AreaPerAxis{
			FMath::Abs(Extents.Y * Extents.Z),
			FMath::Abs(Extents.X * Extents.Z),
			FMath::Abs(Extents.X * Extents.Y)
		};

		// weight the area by the center to center normal
		const Chaos::FReal Area = FMath::Abs(AreaPerAxis.Dot(CenterToCenterNormal));

		return static_cast<float>(Area);
	}
}

namespace Chaos
{
	bool FClusterUnionStressSolver::FNode::IsRootNode() const
	{
		// a root node is a node where a reaction force is happening on the entire structure, causing in return a stress
		// for example the anchored part of a bridge 

		// todo(chaos) : change this logic to be less hacky and be able to handle dynamic structures 
		//				 we could also look for contact point and joints that cause reaction to the 
		return (Particle->IsKinematic() && Particle->GetX().Z < 200.0);
	}

	void FClusterUnionStressSolver::FNode::AddMassContribution(double MassRatio, const FVec3& CenterOfMass)
	{
		SumOfMassRatios += MassRatio;
		WeightedCenterOfMass += (CenterOfMass * MassRatio);
	}

	void FClusterUnionStressSolver::CreateNodeFromClusterUnion()
	{
		const int32 NumChildren = ClusterUnion.ChildParticles.Num();

		NodesById.Reset();
		NodesById.Reserve(NumChildren);
		RootNodeIds.Reset();

		// Create all the node matching the child particles
		for (FPBDRigidParticleHandle* ChildParticle : ClusterUnion.ChildParticles)
		{
			if (ChildParticle)
			{
				if (FPBDRigidClusteredParticleHandle* ClusteredChild = ChildParticle->CastToClustered())
				{
					FNodeId NodeId = ClusteredChild;
					FNode& NewNode = NodesById.Add(NodeId);
					NewNode.Particle = ClusteredChild;
					if (NewNode.IsRootNode())
					{
						RootNodeIds.Add(NodeId);
					}
				}
			}
		}
	}

	void FClusterUnionStressSolver::PrepareForSolve()
	{
		CreateNodeFromClusterUnion();
	}

	void FClusterUnionStressSolver::PropagateValue(const FNode& Node)
	{
		check(Node.Particle);

		for (const TConnectivityEdge<FReal>& Connection : Node.Particle->ConnectivityEdges())
		{
			if (Connection.Sibling)
			{
				const FNodeId OppositeNodeId = Connection.Sibling->CastToClustered();
				if (FNode* OppositeNode = NodesById.Find(OppositeNodeId))
				{
					// value is computed from the accumulated mass
					// if the opposite node value is smaller than the accumulated one, this means it belongs to the branch of another root node
					// and we stop propagation
					// This can happen for example in the middle of a bridge where the mass on both side are equal
					const float NewValue = Node.Value + (float)Connection.Sibling->M();
					if (NewValue < OppositeNode->Value)
					{
						OppositeNode->Value = NewValue;
						PropagateValue(*OppositeNode);
					}
				}
			}
		}
	}

	void FClusterUnionStressSolver::ComputeNodeValues()
	{
		for (const FNodeId RootNodeId: RootNodeIds)
		{
			// todo(chaos) : optimize this to avoid a recursive function
			if (FNode* RootNode = NodesById.Find(RootNodeId))
			{
				RootNode->Value = (float)RootNode->Particle->M();
				PropagateValue(*RootNode);
			}
		}
	}

	FClusterUnionStressSolver::FConnectionEvalResult FClusterUnionStressSolver::EvaluateConnectionStress(double Mass, const FVec3& CenterOfMass, float ConnectionArea, const FVec3& ConnectionCenter, const FVec3& ConnectionNormal)
	{
		FConnectionEvalResult Result;
		if (ConnectionArea < UE_SMALL_NUMBER || Mass < UE_SMALL_NUMBER)
		{
			return Result;
		}

		// strengths are Kg/(cm.s2)
		FChaosPhysicsMaterialStrength Material;

		// use radius as a way to estimate the size cross section of the connection 
		// assuming circular connections for computing moment of inertia for torsion and flexion stress
		const float ConnectionRadius = FMath::Sqrt(FMath::Abs(ConnectionArea / UE_PI));
		const float ConnectionRadiusPower4 = (ConnectionRadius * ConnectionRadius * ConnectionRadius * ConnectionRadius);
		const float AreaMomentOfInertia = ConnectionRadiusPower4 * UE_PI / 4.0f;
		const float PolarMomentOfInertia = ConnectionRadiusPower4 * UE_PI / 2.0f;
		const float OneOverArea = (1.0f / ConnectionArea);

		// collect linear forces on each connected node 
		// we compute the difference to know how much linear stress if resulting from that difference

		// in Kg.cm/s2
		const FVec3 GravityForce = (FVec3::ZAxisVector * (-981 * Mass)); // hack to match concrete
		const FVec3 SumOfForces{ GravityForce };

		const FVec3 ForceLever = CenterOfMass - ConnectionCenter;
		const FVec3 SumOfMoments{ FVec3f::CrossProduct(ForceLever, SumOfForces) };

		// tensile linear component
		const double TensionLinearComponent = SumOfForces.Dot(ConnectionNormal);
		double TensionLinearStress = TensionLinearComponent * OneOverArea;

		// shear linear component
		const FVec3 NormalForce = ConnectionNormal * TensionLinearComponent;
		const FVec3 ShearForce = (SumOfForces - NormalForce);
		const double ShearLinearComponent = ShearForce.Size();
		double ShearLinearStress = ShearLinearComponent * OneOverArea;

		// shear component from torsion 
		const double TorsionMoment = SumOfMoments.Dot(ConnectionNormal);
		double ShearStressFromTorsion = (FMath::Abs(TorsionMoment) * ConnectionRadius) / PolarMomentOfInertia;

		// tensile component from flexion
		const double FlexionMoment = (SumOfMoments - (TorsionMoment * ConnectionNormal)).Size();
		double TensionStressFromFlexion = (FlexionMoment * ConnectionRadius) / AreaMomentOfInertia;

		const double TensileStress = TensionLinearStress + TensionStressFromFlexion;
		const double ShearStress = ShearLinearStress + ShearStressFromTorsion;

		const double CompressionStress = -TensileStress;

		Result.CompressionStressRatio = (float)CompressionStress / (Material.CompressionStrength * ClusterUnionStressSolverStrengthScalar);
		Result.TensileStressRatio = (float)TensileStress / (Material.TensileStrength * ClusterUnionStressSolverStrengthScalar);
		Result.ShearStressRatio = (float)TensileStress / (Material.ShearStrength * ClusterUnionStressSolverStrengthScalar);
		
		// log and debug draw
#if CHAOS_DEBUG_DRAW
		if (bClusterUnionStressSolverEnableDebugDraw)
		{
			constexpr uint8 DepthPriority = 10;
			constexpr float PointThickness = 5.f;
			constexpr float LineThickness = 2.f;

			if (Result.HasFailed())
			{
				FDebugDrawQueue::GetInstance().DrawDebugPoint(CenterOfMass, FColor::Purple, false, UE_KINDA_SMALL_NUMBER, DepthPriority, PointThickness);
				FDebugDrawQueue::GetInstance().DrawDebugLine(CenterOfMass, ConnectionCenter, FColor::Purple, false, UE_KINDA_SMALL_NUMBER, DepthPriority, LineThickness);

				const FString MassStr = FString::FormatAsNumber((int32)Mass);
				FDebugDrawQueue::GetInstance().DrawDebugString(CenterOfMass, MassStr, nullptr, FColor::White, UE_KINDA_SMALL_NUMBER, false, 1.5f);
			}
		}
#endif

		return Result;
	}


	void FClusterUnionStressSolver::EvaluateNode(FNode& Node)
	{
		constexpr uint8 DepthPriority = 10;
		constexpr float PointThickness = 5.f;
		constexpr float LineThickness = 2.f;

		const double TotalMass = ClusterUnion.InternalCluster->M();

		const double MassRatio = (TotalMass > SMALL_NUMBER) ? Node.Particle->M() / TotalMass : 0.f;
		const FVec3 CenterOfMass = Node.Particle->XCom();
		Node.AddMassContribution(MassRatio, CenterOfMass);

		const double EvaluationMass = Node.SumOfMassRatios * TotalMass;
		const FVec3 EvaluationCenterOfMass = Node.GetCenterOfMass();

#if CHAOS_DEBUG_DRAW
		if (bClusterUnionStressSolverEnableDebugDraw)
		{
			FDebugDrawQueue::GetInstance().DrawDebugPoint(Node.Particle->XCom(), FColor::Yellow, false, UE_KINDA_SMALL_NUMBER, DepthPriority, PointThickness * 1.5f);
			const FString ValueStr = FString::SanitizeFloat((float)Node.Value);
			FDebugDrawQueue::GetInstance().DrawDebugString(Node.Particle->XCom(), ValueStr, nullptr, FColor::Purple, UE_KINDA_SMALL_NUMBER, false, 1.0f);
		}
#endif


		// record the indices of the propagating connections
		// we need to know ahead of time how many there is to be able to propagate mass properly between the connections
		TArray<TConnectivityEdge<FReal>> PropagatingConnections;

		const int32 NumConnections = Node.Particle->ConnectivityEdges().Num();
		PropagatingConnections.Reserve(NumConnections);

		// compute areas for the propagating connections 
		float TotalPropagationArea = 0;
		for (const TConnectivityEdge<FReal>& Connection : Node.Particle->ConnectivityEdges())
		{
			if (Connection.Sibling)
			{
				FNodeId OppositeNodeId = Connection.Sibling->CastToClustered();
				if (const FNode* OppositeNode = NodesById.Find(OppositeNodeId))
				{
					// we evaluate following the smaller values as we walk our way back to
					if (OppositeNode->Value < Node.Value)
					{
						// compute the actual overlap surface ( instead of using the strain of the edge - may not be the right area for now ) 
						// todo : we shoud precompute this so we can just use the "strain" property of the connection  

						const float ConnectionArea = ComputeAreaFromBoundingBoxOverlap(Node.Particle->WorldSpaceInflatedBounds(), OppositeNode->Particle->WorldSpaceInflatedBounds());
						TConnectivityEdge<FReal>& PropagatingConnection = PropagatingConnections.Add_GetRef(Connection);
						PropagatingConnection.SetArea(ConnectionArea);
						TotalPropagationArea += ConnectionArea;
					}
				}
			}
		}

		// evaluate and propagate mass properties
		for (const TConnectivityEdge<FReal>& Connection : PropagatingConnections)
		{
			if (Connection.Sibling)
			{
				FNodeId OppositeNodeId = Connection.Sibling->CastToClustered();
				if (FNode* OppositeNode = NodesById.Find(OppositeNodeId))
				{
					ensure(OppositeNode->Value < Node.Value);

					// evalaute stresson the connection 
					const FVec3 ConnectionNormal = (Node.Particle->XCom() - OppositeNode->Particle->XCom()).GetSafeNormal();
					const FVec3 ConnectionCenter = (Node.Particle->XCom() + OppositeNode->Particle->XCom()) * 0.5;
					const float ConnectionArea = Connection.GetArea();
					const double MassContributionRatio = (double)(ConnectionArea / TotalPropagationArea);

					const FConnectionEvalResult ConnectionResult = EvaluateConnectionStress(EvaluationMass * MassContributionRatio, EvaluationCenterOfMass, ConnectionArea, ConnectionCenter, ConnectionNormal);
					if (ConnectionResult.HasFailed())
					{
						FNodeEvalResult& Result = Results.AddDefaulted_GetRef();
						Result.NodeA = Node.GetId();
						Result.NodeB = OppositeNode->GetId();
						Result.ConnectionResult = ConnectionResult;
					}

					// accumulate mass properties
					// mass is distributed equaly between connection
					// todo : see if we could have a better distribution ( based on surface area ? )
					const float ConnectionRatio = 1.f / (float)PropagatingConnections.Num();
					OppositeNode->SumOfMassRatios += Node.SumOfMassRatios * ConnectionRatio;
					OppositeNode->WeightedCenterOfMass += Node.WeightedCenterOfMass * ConnectionRatio;
				}
			}
		}
	}

	void FClusterUnionStressSolver::EvaluateNodes()
	{
		// We want to evaluate the nodes from larges t value to smallest one 
		// as we will be propagating the mass towards the root nodes
		NodesById.ValueSort([](const FNode& A, const FNode& B) { return A.Value> B.Value; });

		Results.Reset();
		for (TPair<FNodeId, FNode>& NodeById : NodesById)
		{
			EvaluateNode(NodeById.Value);
		}
	}
	
	const FClusterUnionStressSolver::FResults& FClusterUnionStressSolver::Solve()
	{
		PrepareForSolve();

		ComputeNodeValues();

		EvaluateNodes();

		return Results;
	}
}



