// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Chaos/Core.h"
#include "Chaos/ParticleHandleFwd.h"

namespace Chaos
{
	struct FClusterUnion;
	//struct FPBDRigidClusteredParticleHandle;

	// this compute stress throughout the structure of a cluster union and find the connections that would break under stress
	struct FClusterUnionStressSolver
	{
	public:
		using FNodeId = const FPBDRigidClusteredParticleHandle*;

		struct FConnectionEvalResult
		{
			float GetMaxStressRatio() const { return FMath::Max3(CompressionStressRatio, TensileStressRatio, ShearStressRatio); }
			bool HasFailed() const { return (CompressionStressRatio >= 1.f) || (TensileStressRatio >= 1.f) || (ShearStressRatio >= 1.f); }
			bool IsGreaterThan(float StressRatio) const { return GetMaxStressRatio() > StressRatio; }

			// Stress ratios below 0 means the failure did not occured, over 1 the failure occured 
			float CompressionStressRatio = 0;
			float TensileStressRatio = 0;
			float ShearStressRatio = 0;
		};

		struct FNodeEvalResult
		{
			FNodeId NodeA = nullptr;
			FNodeId NodeB = nullptr;
			FConnectionEvalResult ConnectionResult;
		};
		using FResults = TArray<FNodeEvalResult>;

		FClusterUnionStressSolver(FClusterUnion& ClusterUnionIn)
			: ClusterUnion(ClusterUnionIn)
		{}

		const FResults& Solve();

	private:
		struct FNode
		{
			FNode()
				: Particle(nullptr)
				, Value(TNumericLimits<float>::Max())
				, SumOfMassRatios(0)
				, WeightedCenterOfMass(FVec3::ZeroVector)
			{}

			FNode(FPBDRigidClusteredParticleHandle* ParticleIn)
				: Particle(ParticleIn)
				, Value(TNumericLimits<float>::Max())
				, SumOfMassRatios(0)
				, WeightedCenterOfMass(FVec3::ZeroVector)
			{}

			FNodeId GetId() const { return Particle; }
			bool IsRootNode() const;
			FVec3 GetCenterOfMass() const { return WeightedCenterOfMass / SumOfMassRatios; }
			void AddMassContribution(double MassRatio, const FVec3& CenterOfMass);

			FPBDRigidClusteredParticleHandle* Particle;

			// this represent the value that will be used to evaluate the nodes from larger to smaller
			float Value;

			// a mass ratio is the ratio of the visited particle masses over the total mass of the cluster union
			// this avoid getting in too large numbers as we accumulate over a large number of nodes
			double SumOfMassRatios;

			// this is the sum of the center of mass weighted by their associated mass ratio contribution ( see above )
			// this allows us to compute the average center of mass at any point 
			FVec3 WeightedCenterOfMass;
		};

		void PrepareForSolve();
		void CreateNodeFromClusterUnion();
		void ComputeNodeValues();
		void PropagateValue(const FNode& Node);
		void EvaluateNodes();
		void EvaluateNode(FNode& Node);
		FConnectionEvalResult EvaluateConnectionStress(double Mass, const FVec3& CenterOfMass, float ConnectionArea, const FVec3& ConnectionCenter, const FVec3& ConnectionNormal);

	private:
		FClusterUnion& ClusterUnion;
		TMap<FNodeId, FNode> NodesById;
		TArray<FNodeId> RootNodeIds;
		FResults Results;
	};
}