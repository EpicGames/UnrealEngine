// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/CollectionEmbeddedSpringConstraintFacade.h"

namespace Chaos::Softs
{
	namespace Private
	{
		template<typename T>
		TArrayView<T> GetArrayView(TManagedArrayAccessor<T>& DataArray, const TManagedArrayAccessor<int32>& StartArray, const TManagedArrayAccessor<int32>& EndArray, int32 ConstraintIndex)
		{
			// Taken from ClothCollection::GetElements.
			// If this pattern is used more in the GeometryCollection facades, we could add this to a utility.
			if (DataArray.IsValid() && StartArray.IsValid() && EndArray.IsValid())
			{
				const int32 Start = StartArray[ConstraintIndex];
				const int32 End = EndArray[ConstraintIndex];
				if (Start != INDEX_NONE && End != INDEX_NONE)
				{
					return TArrayView<T>(DataArray.Modify().GetData() + Start, End - Start + 1);
				}
				checkf(Start == End, TEXT("Only one boundary of the range is set to INDEX_NONE, when both should."));
			}
			return TArrayView<T>();
		}

		template<typename T>
		TConstArrayView<T> GetConstArrayView(const TManagedArrayAccessor<T>& DataArray, const TManagedArrayAccessor<int32>& StartArray, const TManagedArrayAccessor<int32>& EndArray, int32 ConstraintIndex)
		{
			// Taken from ClothCollection::GetElements.
			// If this pattern is used more in the GeometryCollection facades, we could add this to a utility.
			if (DataArray.IsValid() && StartArray.IsValid() && EndArray.IsValid())
			{
				const int32 Start = StartArray[ConstraintIndex];
				const int32 End = EndArray[ConstraintIndex];
				if (Start != INDEX_NONE && End != INDEX_NONE)
				{
					return TConstArrayView<T>(DataArray.Get().GetData() + Start, End - Start + 1);
				}
				checkf(Start == End, TEXT("Only one boundary of the range is set to INDEX_NONE, when both should."));
			}
			return TConstArrayView<T>();
		}

		template<typename T>
		void CopyArrayViewData(const TArrayView<T>& To, const TConstArrayView<T>& From)
		{
			// Taken from ClothCollection::CopyArrayViewData.
			// If this pattern is used more in the GeometryCollection facades, we could add this to a utility.
			check(To.Num() == From.Num());
			for (int32 Index = 0; Index < To.Num(); ++Index)
			{
				To[Index] = From[Index];
			}
		}

		template<typename T>
		void CopyArrayViewDataAndApplyOffset(const TArrayView<TArray<T>>& To, const TConstArrayView<TArray<T>>& From, const T Offset)
		{
			// Taken from ClothCollection::CopyArrayViewDataAndApplyOffset.
			// If this pattern is used more in the GeometryCollection facades, we could add this to a utility.
			check(To.Num() == From.Num());
			for (int32 Index = 0; Index < To.Num(); ++Index)
			{
				To[Index] = From[Index];
				for (T& Value : To[Index])
				{
					Value += Offset;
				}
			}
		}

		void SetIndicesAndWeights(const int32 EndPointNumIndices, TArray<int32>& IndexArray, TArray<float>& WeightArray, const TConstArrayView<int32>& InIndexArray, const TConstArrayView<float>& InWeightArray)
		{
			IndexArray.Reset();
			WeightArray.Reset();
			IndexArray.SetNumZeroed(EndPointNumIndices);
			WeightArray.SetNumZeroed(EndPointNumIndices);
			const int32 InEndPointNumIndices = InIndexArray.Num();
			check(InWeightArray.Num() == InEndPointNumIndices);
			const int32 MinNumIndices = FMath::Min(EndPointNumIndices, InEndPointNumIndices);
			for (int32 Idx = 0; Idx < MinNumIndices; ++Idx)
			{
				IndexArray[Idx] = InIndexArray[Idx];
				WeightArray[Idx] = InWeightArray[Idx];
			}
		}

		/* @return should remove */
		bool CleanIndicesAndWeights(TArray<int32>& IndexArray, TArray<float>& WeightArray)
		{
			check(IndexArray.Num() == WeightArray.Num());
			
			// Return true if any non-zero weighted indices are invalid.
			// Also find first non-zero weighted index.
			bool bAnyZeroWeights = false;
			int32 NonZeroWeightIndex = INDEX_NONE;
			for (int32 Idx = 0; Idx < IndexArray.Num(); ++Idx)
			{
				if (WeightArray[Idx] < UE_SMALL_NUMBER)
				{
					bAnyZeroWeights = true;
				}
				else
				{
					if (IndexArray[Idx] < 0)
					{
						return true;
					}
					NonZeroWeightIndex = IndexArray[Idx];
				}
			}
			if (NonZeroWeightIndex == INDEX_NONE)
			{
				// All weights are Zero. Remove this spring.
				return true;
			}

			if (bAnyZeroWeights)
			{
				// Set all Indices with zero weights to one of the non-zero weighted indices in this spring. This will help with graph coloring and cache.

				for (int32 Idx = 0; Idx < IndexArray.Num(); ++Idx)
				{
					if (WeightArray[Idx] < UE_SMALL_NUMBER)
					{
						IndexArray[Idx] = NonZeroWeightIndex;
					}
				}
			}
			return false;
		}
	}

	// Attributes
	const FName FEmbeddedSpringBaseFacade::SpringLength(TEXT("SpringLength"));
	const FName FEmbeddedSpringBaseFacade::CompressionStiffness(TEXT("CompressionStiffness"));

	const FName FEmbeddedSpringBaseFacade::SpringConstraintGroupName(TEXT("SpringConstraint"));
	const FName FEmbeddedSpringBaseFacade::ConstraintStart(TEXT("ConstraintStart"));
	const FName FEmbeddedSpringBaseFacade::ConstraintEnd(TEXT("ConstraintEnd"));
	const FName FEmbeddedSpringBaseFacade::ConstraintEndPointNumIndices(TEXT("ConstraintEndPointNumIndices"));
	const FName FEmbeddedSpringBaseFacade::ConstraintName(TEXT("ConstraintName"));

	FEmbeddedSpringBaseFacade::FEmbeddedSpringBaseFacade(FManagedArrayCollection& InCollection, const FName& InVerticesGroup)
		: FPositionTargetFacade(InCollection, InVerticesGroup)
		, SpringLengthAttribute(InCollection, SpringLength, GroupName)
		, CompressionStiffnessAttribute(InCollection, CompressionStiffness, GroupName)
		, ConstraintStartAttribute(InCollection, ConstraintStart, SpringConstraintGroupName, GroupName)
		, ConstraintEndAttribute(InCollection, ConstraintEnd, SpringConstraintGroupName, GroupName)
		, ConstraintEndPointNumIndicesAttribute(InCollection, ConstraintEndPointNumIndices, SpringConstraintGroupName)
		, ConstraintNameAttribute(InCollection, ConstraintName, SpringConstraintGroupName)

	{
		DefineSchema();
	}

	FEmbeddedSpringBaseFacade::FEmbeddedSpringBaseFacade(const FManagedArrayCollection& InCollection, const FName& InVerticesGroup)
		: FPositionTargetFacade(InCollection, InVerticesGroup)
		, SpringLengthAttribute(InCollection, SpringLength, GroupName)
		, CompressionStiffnessAttribute(InCollection, CompressionStiffness, GroupName)
		, ConstraintStartAttribute(InCollection, ConstraintStart, SpringConstraintGroupName, GroupName)
		, ConstraintEndAttribute(InCollection, ConstraintEnd, SpringConstraintGroupName, GroupName)
		, ConstraintEndPointNumIndicesAttribute(InCollection, ConstraintEndPointNumIndices, SpringConstraintGroupName)
		, ConstraintNameAttribute(InCollection, ConstraintName, SpringConstraintGroupName)
	{
	}

	bool FEmbeddedSpringBaseFacade::IsValid() const
	{
		return FPositionTargetFacade::IsValid()
			&& SpringLengthAttribute.IsValid() && CompressionStiffnessAttribute.IsValid()
			&& ConstraintStartAttribute.IsValid() && ConstraintEndAttribute.IsValid()
			&& ConstraintEndPointNumIndicesAttribute.IsValid() && ConstraintNameAttribute.IsValid();
	}

	void FEmbeddedSpringBaseFacade::DefineSchema()
	{
		check(!IsConst());
		FPositionTargetFacade::DefineSchema();

		SpringLengthAttribute.Add();
		CompressionStiffnessAttribute.Add();

		ConstraintStartAttribute.Add();
		ConstraintEndAttribute.Add();
		ConstraintEndPointNumIndicesAttribute.Add();
		ConstraintNameAttribute.Add();
	}

	void FEmbeddedSpringFacade::Append(const FEmbeddedSpringFacade& Other, int32 VertexOffset)
	{
		check(!IsConst());
		check(IsValid() && Other.IsValid());
		const int32 StartNumConstraints = GetNumSpringConstraints();
		const int32 OtherNumConstraints = Other.GetNumSpringConstraints();
		SetNumSpringConstraints(StartNumConstraints + OtherNumConstraints);
		for (int32 ConstraintIndex = 0; ConstraintIndex < OtherNumConstraints; ++ConstraintIndex)
		{
			GetSpringConstraint(StartNumConstraints + ConstraintIndex).Initialize(Other.GetSpringConstraintConst(ConstraintIndex), VertexOffset);
		}
	}

	void FEmbeddedSpringFacade::SetNumSpringConstraints(int32 InNumConstraints)
	{
		const int32 NumConstraints = GetNumSpringConstraints();
		for (int32 ConstraintIndex = InNumConstraints; ConstraintIndex < NumConstraints; ++ConstraintIndex)
		{
			GetSpringConstraint(ConstraintIndex).Reset();
		}
		ConstraintStartAttribute.SetNumElements(InNumConstraints);
		for (int32 ConstraintIndex = NumConstraints; ConstraintIndex < InNumConstraints; ++ConstraintIndex)
		{
			GetSpringConstraint(ConstraintIndex).SetDefaults();
		}
	}

	FEmbeddedSpringConstraintFacade FEmbeddedSpringFacade::GetSpringConstraint(int32 ConstraintIndex)
	{
		return IsConst() ? FEmbeddedSpringConstraintFacade(ConstCollection, VerticesGroup, ConstraintIndex) : FEmbeddedSpringConstraintFacade(*Collection, VerticesGroup, ConstraintIndex);
	}

	FEmbeddedSpringConstraintFacade FEmbeddedSpringFacade::GetSpringConstraintConst(int32 ConstraintIndex) const
	{
		return FEmbeddedSpringConstraintFacade(ConstCollection, VerticesGroup, ConstraintIndex);
	}

	int32 FEmbeddedSpringFacade::AddSpringConstraint()
	{
		const int32 ConstraintIndex = GetNumSpringConstraints();
		SetNumSpringConstraints(ConstraintIndex + 1);
		return ConstraintIndex;
	}

	void FEmbeddedSpringFacade::RemoveSpringConstraints(const TArray<int32>& SortedDeletionList)
	{
		check(!IsConst());
		for (const int32 ConstraintToRemove : SortedDeletionList)
		{
			GetSpringConstraint(ConstraintToRemove).Reset();
		}
		Collection->RemoveElements(SpringConstraintGroupName, SortedDeletionList);
	}

	uint32 FEmbeddedSpringFacade::CalculateTypeHash(uint32 PreviousHash) const
	{
		uint32 Hash = PreviousHash;
		Hash = HashCombineFast(Hash, GetNumSpringConstraints());
		for (int32 ConstraintIndex = 0; ConstraintIndex < GetNumSpringConstraints(); ++ConstraintIndex)
		{
			const FEmbeddedSpringConstraintFacade Constraint = GetSpringConstraintConst(ConstraintIndex);
			Hash = Constraint.CalculateTypeHash(Hash);
		}
		return Hash;
	}

	void FEmbeddedSpringFacade::CleanupAndCompactInvalidSprings()
	{
		TArray<int32> SortedConstraintsToRemove;
		for (int32 ConstraintIndex = 0; ConstraintIndex < GetNumSpringConstraints(); ++ConstraintIndex)
		{
			FEmbeddedSpringConstraintFacade Constraint = GetSpringConstraint(ConstraintIndex);
			Constraint.CleanupAndCompactInvalidSprings();
			if (Constraint.GetNumSprings() == 0)
			{
				SortedConstraintsToRemove.Add(ConstraintIndex);
			}
		}
		if (!SortedConstraintsToRemove.IsEmpty())
		{
			RemoveSpringConstraints(SortedConstraintsToRemove);
		}
	}

	FEmbeddedSpringConstraintFacade::FEmbeddedSpringConstraintFacade(FManagedArrayCollection& InCollection, const FName& InVerticesGroup, int32 InConstraintIndex)
		:Base(InCollection, InVerticesGroup)
		, ConstraintIndex(InConstraintIndex)
	{
		check(IsValid());
		check(ConstraintIndex >= 0 && ConstraintIndex < ConstraintStartAttribute.Num());
	}

	FEmbeddedSpringConstraintFacade::FEmbeddedSpringConstraintFacade(const FManagedArrayCollection& InCollection, const FName& InVerticesGroup, int32 InConstraintIndex)
		: Base(InCollection, InVerticesGroup)
		, ConstraintIndex(InConstraintIndex)
	{
		check(IsValid());
		check(ConstraintIndex >= 0 && ConstraintIndex < ConstraintStartAttribute.Num());
	}

	void FEmbeddedSpringConstraintFacade::SetDefaults()
	{
		check(!IsConst());
		ConstraintStartAttribute.ModifyAt(ConstraintIndex, INDEX_NONE);
		ConstraintEndAttribute.ModifyAt(ConstraintIndex, INDEX_NONE);
		SetConstraintName(TEXT(""));
	}

	void FEmbeddedSpringConstraintFacade::Reset()
	{
		SetNumSprings(0);
		SetDefaults();
	}

	int32 FEmbeddedSpringConstraintFacade::GetNumSprings() const
	{
		// Taken from ClothCollection::GetNumElements.
		// If this pattern is used more in the GeometryCollection facades, we could add this to a utility.
		if (ConstraintStartAttribute.IsValid() && ConstraintEndAttribute.IsValid())
		{
			const int32 Start = ConstraintStartAttribute[ConstraintIndex];
			const int32 End = ConstraintEndAttribute[ConstraintIndex];
			if (Start != INDEX_NONE && End != INDEX_NONE)
			{
				return End - Start + 1;
			}
			checkf(Start == End, TEXT("Only one boundary of the range is set to INDEX_NONE, when both should."));
		}
		return 0;
	}

	void FEmbeddedSpringConstraintFacade::RemoveSprings(const TArray<int32>& SortedDeletionList)
	{
		check(!IsConst());
		if (SortedDeletionList.IsEmpty())
		{
			return;
		}
		TArray<int32> GlobalIndexSortedDeletionList;
		const int32 Offset = ConstraintStartAttribute[ConstraintIndex];
		GlobalIndexSortedDeletionList.SetNumUninitialized(SortedDeletionList.Num());
		for (int32 Idx = 0; Idx < SortedDeletionList.Num(); ++Idx)
		{
			GlobalIndexSortedDeletionList[Idx] = SortedDeletionList[Idx] + Offset;
		}

		// Taken from ClothCollection::RemoveElements.
		// If this pattern is used more in the GeometryCollection facades, we could add this to a utility.
		check(ConstraintStartAttribute.IsValid() && ConstraintEndAttribute.IsValid());
		TManagedArray<int32>& ConstraintStartArray = ConstraintStartAttribute.Modify();
		TManagedArray<int32>& ConstraintEndArray = ConstraintEndAttribute.Modify();

		int32& Start = ConstraintStartArray[ConstraintIndex];
		int32& End = ConstraintEndArray[ConstraintIndex];
		check(Start != INDEX_NONE && End != INDEX_NONE);

		const int32 OrigStart = Start;
		const int32 OrigNumElements = End - Start + 1;

		check(GlobalIndexSortedDeletionList[0] >= Start);
		check(GlobalIndexSortedDeletionList.Last() <= End);
		check(OrigNumElements >= GlobalIndexSortedDeletionList.Num());

		Collection->RemoveElements(GroupName, GlobalIndexSortedDeletionList);

		if (GlobalIndexSortedDeletionList.Num() == OrigNumElements)
		{
			Start = End = INDEX_NONE;
		}
		else
		{
			const int32 NewNumElements = OrigNumElements - GlobalIndexSortedDeletionList.Num();
			const int32 NewEnd = OrigStart + NewNumElements - 1;
			check(Start == OrigStart || Start == INDEX_NONE);
			check(End == NewEnd || End == INDEX_NONE);
			Start = OrigStart;
			End = NewEnd;
		}
	}

	void FEmbeddedSpringConstraintFacade::SetNumSprings(int32 InNumSprings)
	{
		// Taken from ClothCollection::SetNumElements.
		// If this pattern is used more in the GeometryCollection facades, we could add this to a utility.
		check(!IsConst());
		check(InNumSprings >= 0);
		check(ConstraintStartAttribute.IsValid() && ConstraintEndAttribute.IsValid());
		TManagedArray<int32>& ConstraintStartArray = ConstraintStartAttribute.Modify();
		TManagedArray<int32>& ConstraintEndArray = ConstraintEndAttribute.Modify();
		
		int32& Start = ConstraintStartArray[ConstraintIndex];
		int32& End = ConstraintEndArray[ConstraintIndex];
		check(Start != INDEX_NONE || End == INDEX_NONE);  // Best to avoid situations where only one boundary of the range is set to INDEX_NONE

		const int32 NumSprings = (Start == INDEX_NONE) ? 0 : End - Start + 1;
		if (const int32 Delta = InNumSprings - NumSprings)
		{
			if (Delta > 0)
			{
				// Find a previous valid index range to insert after when the range is empty
				auto ComputeEnd = [this, &ConstraintEndArray]()->int32
					{
						for (int32 Index = ConstraintIndex; Index >= 0; --Index)
						{
							if (ConstraintEndArray[Index] != INDEX_NONE)
							{
								return ConstraintEndArray[Index];
							}
						}
						return INDEX_NONE;
					};

				// Grow the array
				const int32 Position = ComputeEnd() + 1;
				Collection->InsertElements(Delta, Position, GroupName);

				// Update Start/End
				if (!NumSprings)
				{
					Start = Position;
				}
				End = Start + InNumSprings - 1;
			}
			else
			{
				// Shrink the array
				const int32 Position = Start + InNumSprings;
				Collection->RemoveElements(GroupName, -Delta, Position);

				// Update Start/End
				if (InNumSprings)
				{
					End = Position - 1;
				}
				else
				{
					End = Start = INDEX_NONE;  // It is important to set the start & end to INDEX_NONE so that they never get automatically re-indexed by the managed array collection
				}
			}
		}
	}

	void FEmbeddedSpringConstraintFacade::Initialize(const TConstArrayView<FIntVector2>& EndPoints, const TConstArrayView<float>& InSpringLength,
		const TConstArrayView<float>& InExtensionStiffnessWeight, const TConstArrayView<float>& InCompressionStiffnessWeight, const TConstArrayView<float>& InDampingWeight, const FString& InConstraintName)
	{
		check(!IsConst());
		Reset();

		//~ Spring Constraint Group
		SetConstraintEndPointNumIndices(FUintVector2(1, 1));
		SetConstraintName(InConstraintName);
		Append(EndPoints, InSpringLength, InExtensionStiffnessWeight, InCompressionStiffnessWeight, InDampingWeight);
	}

	void FEmbeddedSpringConstraintFacade::Append(const TConstArrayView<FIntVector2>&EndPoints, const TConstArrayView<float>&InSpringLength,
		const TConstArrayView<float>&InExtensionStiffnessWeight, const TConstArrayView<float>&InCompressionStiffnessWeight, const TConstArrayView<float>&InDampingWeight)
	{
		if (GetConstraintEndPointNumIndices() != FUintVector2(1, 1))
		{
			return;
		}

		const int32 OrigNumSprings = GetNumSprings();

		//~ Group PositionTargets
		const int32 InNumSprings = EndPoints.Num();
		check(InNumSprings == InSpringLength.Num());
		SetNumSprings(OrigNumSprings + InNumSprings);
		TArrayView<TArray<int32>> SourceIndexArray = GetSourceIndex().Right(InNumSprings);
		TArrayView<TArray<int32>> TargetIndexArray = GetTargetIndex().Right(InNumSprings);
		TArrayView<TArray<float>> SourceWeightArray = GetSourceWeights().Right(InNumSprings);
		TArrayView<TArray<float>> TargetWeightArray = GetTargetWeights().Right(InNumSprings);
		TArrayView<float> SpringLengthArray = GetSpringLength().Right(InNumSprings);
		TArrayView<float> ExtensionStiffnessArray = GetExtensionStiffness().Right(InNumSprings);
		TArrayView<float> CompressionStiffnessArray = GetCompressionStiffness().Right(InNumSprings);
		TArrayView<float> DampingArray = GetDamping().Right(InNumSprings);
		for (int32 SpringIdx = 0; SpringIdx < InNumSprings; ++SpringIdx)
		{
			SourceIndexArray[SpringIdx] = { EndPoints[SpringIdx][0] };
			TargetIndexArray[SpringIdx] = { EndPoints[SpringIdx][1] };
			SourceWeightArray[SpringIdx] = { 1.f };
			TargetWeightArray[SpringIdx] = { 1.f };
			SpringLengthArray[SpringIdx] = InSpringLength[SpringIdx];
			ExtensionStiffnessArray[SpringIdx] = InExtensionStiffnessWeight.IsValidIndex(SpringIdx) ? InExtensionStiffnessWeight[SpringIdx] : 0.f;
			CompressionStiffnessArray[SpringIdx] = InCompressionStiffnessWeight.IsValidIndex(SpringIdx) ? InCompressionStiffnessWeight[SpringIdx] : 0.f;
			DampingArray[SpringIdx] = InDampingWeight.IsValidIndex(SpringIdx) ? InDampingWeight[SpringIdx] : 0.f;
		}
	}

	void FEmbeddedSpringConstraintFacade::Initialize(const FUintVector2& EndPointNumIndices, const TConstArrayView<TArray<int32>>& InSourceIndices, const TConstArrayView<TArray<float>>& InSourceWeights,
		const TConstArrayView<TArray<int32>>& InTargetIndices, const TConstArrayView<TArray<float>>& InTargetWeights,
		const TConstArrayView<float>& InSpringLength,
		const TConstArrayView<float>& InExtensionStiffnessWeight, const TConstArrayView<float>& InCompressionStiffnessWeight, const TConstArrayView<float>& InDampingWeight, const FString& InConstraintName)
	{
		check(!IsConst());
		Reset();

		//~ Spring Constraint Group
		SetConstraintEndPointNumIndices(EndPointNumIndices);
		SetConstraintName(InConstraintName);

		Append(InSourceIndices, InSourceWeights, InTargetIndices, InTargetWeights, InSpringLength, InExtensionStiffnessWeight, InCompressionStiffnessWeight, InDampingWeight);
	}

	void FEmbeddedSpringConstraintFacade::Append(const TConstArrayView<TArray<int32>>& InSourceIndices, const TConstArrayView<TArray<float>>& InSourceWeights,
		const TConstArrayView<TArray<int32>>& InTargetIndices, const TConstArrayView<TArray<float>>& InTargetWeights,
		const TConstArrayView<float>& InSpringLength,
		const TConstArrayView<float>& InExtensionStiffnessWeight, const TConstArrayView<float>& InCompressionStiffnessWeight, const TConstArrayView<float>& InDampingWeight)
	{

		const int32 OrigNumSprings = GetNumSprings();

		//~ Group PositionTargets
		const int32 InNumSprings = InSourceIndices.Num();
		check(InNumSprings == InSourceWeights.Num());
		check(InNumSprings == InTargetIndices.Num());
		check(InNumSprings == InTargetWeights.Num());
		check(InNumSprings == InSpringLength.Num());
		SetNumSprings(OrigNumSprings + InNumSprings);
		TArrayView<float> SpringLengthArray = GetSpringLength().Right(InNumSprings);
		TArrayView<float> ExtensionStiffnessArray = GetExtensionStiffness().Right(InNumSprings);
		TArrayView<float> CompressionStiffnessArray = GetCompressionStiffness().Right(InNumSprings);
		TArrayView<float> DampingArray = GetDamping().Right(InNumSprings);
		for (int32 SpringIdx = 0; SpringIdx < InNumSprings; ++SpringIdx)
		{
			SetIndicesAndWeights(SpringIdx + OrigNumSprings, TConstArrayView<int32>(InSourceIndices[SpringIdx]), TConstArrayView<float>(InSourceWeights[SpringIdx]),
				TConstArrayView<int32>(InTargetIndices[SpringIdx]), TConstArrayView<float>(InTargetWeights[SpringIdx]));
			SpringLengthArray[SpringIdx] = InSpringLength[SpringIdx];
			ExtensionStiffnessArray[SpringIdx] = InExtensionStiffnessWeight.IsValidIndex(SpringIdx) ? InExtensionStiffnessWeight[SpringIdx] : 0.f;
			CompressionStiffnessArray[SpringIdx] = InCompressionStiffnessWeight.IsValidIndex(SpringIdx) ? InCompressionStiffnessWeight[SpringIdx] : 0.f;
			DampingArray[SpringIdx] = InDampingWeight.IsValidIndex(SpringIdx) ? InDampingWeight[SpringIdx] : 0.f;
		}
	}

	void FEmbeddedSpringConstraintFacade::Initialize(const FEmbeddedSpringConstraintFacade& Other, const int32 VertexOffset)
	{
		check(!IsConst());
		check(VerticesGroup == Other.VerticesGroup);
		Reset();

		//~ Spring Constraint Group
		SetConstraintEndPointNumIndices(Other.GetConstraintEndPointNumIndices());
		SetConstraintName(Other.GetConstraintName());

		//~ Group PositionTargets
		SetNumSprings(Other.GetNumSprings());
		Private::CopyArrayViewDataAndApplyOffset(GetSourceIndex(), Other.GetSourceIndexConst(), VertexOffset);
		Private::CopyArrayViewDataAndApplyOffset(GetTargetIndex(), Other.GetTargetIndexConst(), VertexOffset);
		Private::CopyArrayViewData(GetSourceWeights(), Other.GetSourceWeightsConst());
		Private::CopyArrayViewData(GetTargetWeights(), Other.GetTargetWeightsConst());
		Private::CopyArrayViewData(GetSpringLength(), Other.GetSpringLengthConst());
		Private::CopyArrayViewData(GetExtensionStiffness(), Other.GetExtensionStiffnessConst());
		Private::CopyArrayViewData(GetCompressionStiffness(), Other.GetCompressionStiffnessConst());
		Private::CopyArrayViewData(GetDamping(), Other.GetDampingConst());
	}

	void FEmbeddedSpringConstraintFacade::Append(const FEmbeddedSpringConstraintFacade& Other, const int32 VertexOffset)
	{
		if (Other.GetNumSprings() == 0)
		{
			return;
		}

		check(!IsConst());
		check(VerticesGroup == Other.VerticesGroup);
		check(GetConstraintEndPointNumIndices() == Other.GetConstraintEndPointNumIndices());

		//~ Group PositionTargets
		const int32 OrigNumSprings = GetNumSprings();
		const int32 OtherNumSprings = Other.GetNumSprings();
		SetNumSprings(OrigNumSprings + OtherNumSprings);
		Private::CopyArrayViewDataAndApplyOffset(TArrayView<TArray<int32>>(GetSourceIndex().GetData() + OrigNumSprings, OtherNumSprings), Other.GetSourceIndexConst(), VertexOffset);
		Private::CopyArrayViewDataAndApplyOffset(TArrayView<TArray<int32>>(GetTargetIndex().GetData() + OrigNumSprings, OtherNumSprings), Other.GetTargetIndexConst(), VertexOffset);
		Private::CopyArrayViewData(TArrayView<TArray<float>>(GetSourceWeights().GetData() + OrigNumSprings, OtherNumSprings), Other.GetSourceWeightsConst());
		Private::CopyArrayViewData(TArrayView<TArray<float>>(GetTargetWeights().GetData() + OrigNumSprings, OtherNumSprings), Other.GetTargetWeightsConst());
		Private::CopyArrayViewData(TArrayView<float>(GetSpringLength().GetData() + OrigNumSprings, OtherNumSprings), Other.GetSpringLengthConst());
		Private::CopyArrayViewData(TArrayView<float>(GetExtensionStiffness().GetData() + OrigNumSprings, OtherNumSprings), Other.GetExtensionStiffnessConst());
		Private::CopyArrayViewData(TArrayView<float>(GetCompressionStiffness().GetData() + OrigNumSprings, OtherNumSprings), Other.GetCompressionStiffnessConst());
		Private::CopyArrayViewData(TArrayView<float>(GetDamping().GetData() + OrigNumSprings, OtherNumSprings), Other.GetDampingConst());
	}

	TArrayView<float> FEmbeddedSpringConstraintFacade::GetSpringLength()
	{
		return Private::GetArrayView(SpringLengthAttribute, ConstraintStartAttribute, ConstraintEndAttribute, ConstraintIndex);
	}

	TConstArrayView<float> FEmbeddedSpringConstraintFacade::GetSpringLengthConst() const
	{
		return Private::GetConstArrayView(SpringLengthAttribute, ConstraintStartAttribute, ConstraintEndAttribute, ConstraintIndex);
	}

	TArrayView<float> FEmbeddedSpringConstraintFacade::GetExtensionStiffness()
	{
		return Private::GetArrayView(StiffnessAttribute, ConstraintStartAttribute, ConstraintEndAttribute, ConstraintIndex);
	}

	TConstArrayView<float> FEmbeddedSpringConstraintFacade::GetExtensionStiffnessConst() const
	{
		return Private::GetConstArrayView(StiffnessAttribute, ConstraintStartAttribute, ConstraintEndAttribute, ConstraintIndex);
	}

	TArrayView<float> FEmbeddedSpringConstraintFacade::GetCompressionStiffness()
	{
		return Private::GetArrayView(CompressionStiffnessAttribute, ConstraintStartAttribute, ConstraintEndAttribute, ConstraintIndex);
	}

	TConstArrayView<float> FEmbeddedSpringConstraintFacade::GetCompressionStiffnessConst() const
	{
		return Private::GetConstArrayView(CompressionStiffnessAttribute, ConstraintStartAttribute, ConstraintEndAttribute, ConstraintIndex);
	}

	TArrayView<float> FEmbeddedSpringConstraintFacade::GetDamping()
	{
		return Private::GetArrayView(DampingAttribute, ConstraintStartAttribute, ConstraintEndAttribute, ConstraintIndex);
	}

	TConstArrayView<float> FEmbeddedSpringConstraintFacade::GetDampingConst() const
	{
		return Private::GetConstArrayView(DampingAttribute, ConstraintStartAttribute, ConstraintEndAttribute, ConstraintIndex);
	}

	TArrayView<TArray<int32>> FEmbeddedSpringConstraintFacade::GetSourceIndex()
	{
		return Private::GetArrayView(SourceIndexAttribute, ConstraintStartAttribute, ConstraintEndAttribute, ConstraintIndex);
	}

	TConstArrayView<TArray<int32>> FEmbeddedSpringConstraintFacade::GetSourceIndexConst() const
	{
		return Private::GetConstArrayView(SourceIndexAttribute, ConstraintStartAttribute, ConstraintEndAttribute, ConstraintIndex);
	}

	TArrayView<TArray<float>> FEmbeddedSpringConstraintFacade::GetSourceWeights()
	{
		return Private::GetArrayView(SourceWeightsAttribute, ConstraintStartAttribute, ConstraintEndAttribute, ConstraintIndex);
	}

	TConstArrayView<TArray<float>> FEmbeddedSpringConstraintFacade::GetSourceWeightsConst() const
	{
		return Private::GetConstArrayView(SourceWeightsAttribute, ConstraintStartAttribute, ConstraintEndAttribute, ConstraintIndex);
	}

	TArrayView<TArray<int32>> FEmbeddedSpringConstraintFacade::GetTargetIndex()
	{
		return Private::GetArrayView(TargetIndexAttribute, ConstraintStartAttribute, ConstraintEndAttribute, ConstraintIndex);
	}

	TConstArrayView<TArray<int32>> FEmbeddedSpringConstraintFacade::GetTargetIndexConst() const
	{
		return Private::GetConstArrayView(TargetIndexAttribute, ConstraintStartAttribute, ConstraintEndAttribute, ConstraintIndex);
	}

	TArrayView<TArray<float>> FEmbeddedSpringConstraintFacade::GetTargetWeights()
	{
		return Private::GetArrayView(TargetWeightsAttribute, ConstraintStartAttribute, ConstraintEndAttribute, ConstraintIndex);
	}

	TConstArrayView<TArray<float>> FEmbeddedSpringConstraintFacade::GetTargetWeightsConst() const
	{
		return Private::GetConstArrayView(TargetWeightsAttribute, ConstraintStartAttribute, ConstraintEndAttribute, ConstraintIndex);
	}

	void FEmbeddedSpringConstraintFacade::SetIndicesAndWeights(const int32 SpringIndex, const TConstArrayView<int32>& InSourceIndices, const TConstArrayView<float>& InSourceWeights,
		const TConstArrayView<int32>& InTargetIndices, const TConstArrayView<float>& InTargetWeights)
	{
		const FUintVector2 EndPointNumIndices = GetConstraintEndPointNumIndices();
		Private::SetIndicesAndWeights(EndPointNumIndices[0], GetSourceIndex()[SpringIndex], GetSourceWeights()[SpringIndex], InSourceIndices, InSourceWeights);
		Private::SetIndicesAndWeights(EndPointNumIndices[1], GetTargetIndex()[SpringIndex], GetTargetWeights()[SpringIndex], InTargetIndices, InTargetWeights);
	}

	uint32 FEmbeddedSpringConstraintFacade::CalculateTypeHash(uint32 PreviousHash) const
	{
		uint32 Hash = PreviousHash;
		Hash = HashCombineFast(Hash, GetTypeHash(GetConstraintEndPointNumIndices()));
		Hash = GetArrayHash(GetSourceIndexConst().GetData(), GetSourceIndexConst().Num(), Hash);
		Hash = GetArrayHash(GetSourceWeightsConst().GetData(), GetSourceWeightsConst().Num(), Hash);
		Hash = GetArrayHash(GetTargetIndexConst().GetData(), GetTargetIndexConst().Num(), Hash);
		Hash = GetArrayHash(GetTargetWeightsConst().GetData(), GetTargetWeightsConst().Num(), Hash);
		Hash = GetArrayHash(GetSpringLengthConst().GetData(), GetSpringLengthConst().Num(), Hash);
		Hash = GetArrayHash(GetExtensionStiffnessConst().GetData(), GetExtensionStiffnessConst().Num(), Hash);
		Hash = GetArrayHash(GetCompressionStiffnessConst().GetData(), GetCompressionStiffnessConst().Num(), Hash);
		Hash = GetArrayHash(GetDampingConst().GetData(), GetDampingConst().Num(), Hash);
		return Hash;
	}

	void FEmbeddedSpringConstraintFacade::CleanupAndCompactInvalidSprings()
	{
		TArray<int32> SortedSpringsToRemove;
		const int32 NumSprings = GetNumSprings();

		for (int32 Index = 0; Index < NumSprings; ++Index)
		{
			if (Private::CleanIndicesAndWeights(GetSourceIndex()[Index], GetSourceWeights()[Index]))
			{
				SortedSpringsToRemove.Add(Index);
				continue;
			}
			if (Private::CleanIndicesAndWeights(GetTargetIndex()[Index], GetTargetWeights()[Index]))
			{
				SortedSpringsToRemove.Add(Index);
			}
		}

		if (!SortedSpringsToRemove.IsEmpty())
		{
			RemoveSprings(SortedSpringsToRemove);
		}
	}
}
