// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/CollectionPropertyFacade.h"
#include "GeometryCollection/ManagedArrayCollection.h"

namespace Chaos::Softs
{
	namespace PropertyFacadeNames
	{
		// Attribute groups, predefined data member of the collection
		static const FName PropertyGroup("Property");
		static const FName KeyNameName("KeyName");  // Property key, name to look for
		static const FName LowValueName("LowValue");  // Boolean, 24 bit integer (max 16777215), float, or vector value, or value of the lowest weight on the weight map if any
		static const FName HighValueName("HighValue");  // Boolean, 24 bit integer (max 16777215), float, or vector value of the highest weight on the weight map if any
		static const FName StringValueName("StringValue");  // String value, or weight map name, ...etc.
		static const FName FlagsName("Flags");  // Whether this property is enabled, animatable, ...etc.
		static const FName PropertyGlobalGroup("PropertyGlobal");  // Contains all single elements of this facade
		static const FName DirtyName("Dirty");  // Dirty flags for this facade at the collection level

		// Previous name for String-based Key
		static const FName KeyName("Key");
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FCollectionPropertyConstFacade::~FCollectionPropertyConstFacade() = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	FCollectionPropertyConstFacade::FCollectionPropertyConstFacade(const TSharedPtr<const FManagedArrayCollection>& InManagedArrayCollection)
		: ManagedArrayCollection(InManagedArrayCollection)
	{
		UpdateArrays();
		RebuildKeyIndices();
	}

	FCollectionPropertyConstFacade::FCollectionPropertyConstFacade(const TSharedPtr<const FManagedArrayCollection>& InManagedArrayCollection, ENoInit)
		: ManagedArrayCollection(InManagedArrayCollection)
	{
	}

	bool FCollectionPropertyConstFacade::IsValid() const
	{
		return
			ManagedArrayCollection->HasAttribute(PropertyFacadeNames::KeyNameName, PropertyFacadeNames::PropertyGroup) &&
			ManagedArrayCollection->HasAttribute(PropertyFacadeNames::LowValueName, PropertyFacadeNames::PropertyGroup) &&
			ManagedArrayCollection->HasAttribute(PropertyFacadeNames::HighValueName, PropertyFacadeNames::PropertyGroup) &&
			ManagedArrayCollection->HasAttribute(PropertyFacadeNames::StringValueName, PropertyFacadeNames::PropertyGroup) &&
			ManagedArrayCollection->HasAttribute(PropertyFacadeNames::FlagsName, PropertyFacadeNames::PropertyGroup) &&
			ManagedArrayCollection->HasAttribute(PropertyFacadeNames::DirtyName, PropertyFacadeNames::PropertyGlobalGroup);
	}

	bool FCollectionPropertyConstFacade::IsDirty() const
	{
		if (const TManagedArray<bool>* const DirtyAttribute = ManagedArrayCollection->FindAttributeTyped<bool>(PropertyFacadeNames::DirtyName, PropertyFacadeNames::PropertyGlobalGroup);
			DirtyAttribute && DirtyAttribute->Num())
		{
			return (*DirtyAttribute)[0];
		}
		return false;
	}

	void FCollectionPropertyConstFacade::UpdateArrays()
	{
		KeyNameArray = GetArray<FName>(PropertyFacadeNames::KeyNameName);
		LowValueArray = GetArray<FVector3f>(PropertyFacadeNames::LowValueName);
		HighValueArray = GetArray<FVector3f>(PropertyFacadeNames::HighValueName);
		StringValueArray = GetArray<FString>(PropertyFacadeNames::StringValueName);
		FlagsArray = GetArray<uint8>(PropertyFacadeNames::FlagsName);
	}
	
	void FCollectionPropertyConstFacade::RebuildKeyIndices()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FCollectionPropertyConstFacade_RebuildKeyIndices);
		// Create a fast access search map (although it might only be faster for a large enough number of properties)
		const int32 NumKeys = KeyNameArray.Num();
		KeyNameIndices.Empty(NumKeys);
		for (int32 Index = 0; Index < NumKeys; ++Index)
		{
			KeyNameIndices.Emplace(KeyNameArray[Index], Index);
		}
	}

	template<typename T, typename ElementType>
	T FCollectionPropertyConstFacade::GetValue(int32 KeyIndex, const TConstArrayView<ElementType>& ValueArray) const
	{
		return (T)ValueArray[KeyIndex];
	}
	template CHAOS_API FVector3f FCollectionPropertyConstFacade::GetValue<FVector3f, FVector3f>(int32 KeyIndex, const TConstArrayView<FVector3f>& ValueArray) const;
	template CHAOS_API const FVector3f& FCollectionPropertyConstFacade::GetValue<const FVector3f&, FVector3f>(int32 KeyIndex, const TConstArrayView<FVector3f>& ValueArray) const;
	template CHAOS_API FString FCollectionPropertyConstFacade::GetValue<FString, FString>(int32 KeyIndex, const TConstArrayView<FString>& ValueArray) const;
	template CHAOS_API const FString& FCollectionPropertyConstFacade::GetValue<const FString&, FString>(int32 KeyIndex, const TConstArrayView<FString>& ValueArray) const;
	template CHAOS_API FName FCollectionPropertyConstFacade::GetValue<FName, FName>(int32 KeyIndex, const TConstArrayView<FName>& ValueArray) const;
	template CHAOS_API const FName& FCollectionPropertyConstFacade::GetValue<const FName&, FName>(int32 KeyIndex, const TConstArrayView<FName>& ValueArray) const;
	template CHAOS_API uint8 FCollectionPropertyConstFacade::GetValue<uint8, uint8>(int32 KeyIndex, const TConstArrayView<uint8>& ValueArray) const;
	template CHAOS_API ECollectionPropertyFlags FCollectionPropertyConstFacade::GetValue<ECollectionPropertyFlags, uint8>(int32 KeyIndex, const TConstArrayView<uint8>& ValueArray) const;

	template<> CHAOS_API
	bool FCollectionPropertyConstFacade::GetValue<bool, FVector3f>(int32 KeyIndex, const TConstArrayView<FVector3f>& ValueArray) const
	{
		return (bool)ValueArray[KeyIndex].X;
	}

	template<> CHAOS_API
	int32 FCollectionPropertyConstFacade::GetValue<int32, FVector3f>(int32 KeyIndex, const TConstArrayView<FVector3f>& ValueArray) const
	{
		return (int32)ValueArray[KeyIndex].X;
	}

	template<> CHAOS_API
	float FCollectionPropertyConstFacade::GetValue<float, FVector3f>(int32 KeyIndex, const TConstArrayView<FVector3f>& ValueArray) const
	{
		return ValueArray[KeyIndex].X;
	}

	template <typename T>
	TConstArrayView<T> FCollectionPropertyConstFacade::GetArray(const FName& Name) const
	{
		const TManagedArray<T>* ManagedArray = ManagedArrayCollection->FindAttributeTyped<T>(Name, PropertyFacadeNames::PropertyGroup);
		return ManagedArray ? TConstArrayView<T>(ManagedArray->GetConstArray()) : TConstArrayView<T>();
	}

	FCollectionPropertyFacade::FCollectionPropertyFacade(const TSharedPtr<FManagedArrayCollection>& InManagedArrayCollection)
		: FCollectionPropertyConstFacade(InManagedArrayCollection, NoInit)
	{
		UpdateArrays();
		RebuildKeyIndices();
	}

	FCollectionPropertyFacade::FCollectionPropertyFacade(const TSharedPtr<FManagedArrayCollection>& InManagedArrayCollection, ENoInit)
		: FCollectionPropertyConstFacade(InManagedArrayCollection, NoInit)
	{
	}

	TArrayView<FName> FCollectionPropertyFacade::ModifyKeyNameArray()
	{
		if (KeyNameArray.Num())
		{
			MakeDirty();
			TManagedArray<FName>& KeyNames = GetManagedArrayCollection()->ModifyAttribute<FName>(
				PropertyFacadeNames::KeyNameName, PropertyFacadeNames::PropertyGroup);
			return TArrayView<FName>(KeyNames.GetData(), KeyNames.Num());
		}
		return TArrayView<FName>();
	}

	TArrayView<FVector3f> FCollectionPropertyFacade::ModifyLowValueArray()
	{
		if (LowValueArray.Num())
		{
			MakeDirty();
			TManagedArray<FVector3f>& LowValues = GetManagedArrayCollection()->ModifyAttribute<FVector3f>(
				PropertyFacadeNames::LowValueName, PropertyFacadeNames::PropertyGroup);
			return TArrayView<FVector3f>(LowValues.GetData(), LowValues.Num());
		}
		return TArrayView<FVector3f>();
	}

	TArrayView<FVector3f> FCollectionPropertyFacade::ModifyHighValueArray()
	{
		if (HighValueArray.Num())
		{
			MakeDirty();
			TManagedArray<FVector3f>& HighValues = GetManagedArrayCollection()->ModifyAttribute<FVector3f>(
				PropertyFacadeNames::HighValueName, PropertyFacadeNames::PropertyGroup);
			return TArrayView<FVector3f>(HighValues.GetData(), HighValues.Num());
		}
		return TArrayView<FVector3f>();
	}

	TArrayView<FString> FCollectionPropertyFacade::ModifyStringValueArray()
	{
		if (StringValueArray.Num())
		{
			MakeDirty();
			TManagedArray<FString>& StringValues = GetManagedArrayCollection()->ModifyAttribute<FString>(
				PropertyFacadeNames::StringValueName, PropertyFacadeNames::PropertyGroup);
			return TArrayView<FString>(StringValues.GetData(), StringValues.Num());
		}
		return TArrayView<FString>();
	}

	TArrayView<ECollectionPropertyFlags> FCollectionPropertyFacade::ModifyFlagsArray()
	{
		if (FlagsArray.Num())
		{
			MakeDirty();
			TManagedArray<uint8>& FlagBytes = GetManagedArrayCollection()->ModifyAttribute<uint8>(
				PropertyFacadeNames::FlagsName, PropertyFacadeNames::PropertyGroup);
			return TArrayView<ECollectionPropertyFlags>(
				reinterpret_cast<ECollectionPropertyFlags*>(FlagBytes.GetData()), FlagBytes.Num());
		}
		return TArrayView<ECollectionPropertyFlags>();
	}

	void FCollectionPropertyFacade::SetFlags(int32 KeyIndex, ECollectionPropertyFlags Flags)
	{
		// Cannot set string dirty without also dirtying the property
		Flags |= EnumHasAnyFlags(Flags, ECollectionPropertyFlags::StringDirty) ? ECollectionPropertyFlags::Dirty : ECollectionPropertyFlags::None;
		// Cannot remove the Dirty, StringDirty, and Intrinsic flags
		Flags |= (ECollectionPropertyFlags)FlagsArray[KeyIndex] & (ECollectionPropertyFlags::Dirty | ECollectionPropertyFlags::StringDirty | ECollectionPropertyFlags::Intrinsic | ECollectionPropertyFlags::Interpolable);

		SetValue(KeyIndex, ModifyFlagsArray(), Flags);
	}

	void FCollectionPropertyFacade::ClearDirtyFlags()
	{
		for (ECollectionPropertyFlags& Flags : ModifyFlagsArray())
		{
			EnumRemoveFlags(Flags, ECollectionPropertyFlags::StringDirty | ECollectionPropertyFlags::Dirty);
		}
		if (TManagedArray<bool>* const DirtyAttribute = GetManagedArrayCollection()->FindAttributeTyped<bool>(PropertyFacadeNames::DirtyName, PropertyFacadeNames::PropertyGlobalGroup);
			DirtyAttribute && DirtyAttribute->Num())
		{
			(*DirtyAttribute)[0] = false;
		}
	}

	void FCollectionPropertyFacade::MakeDirty()
	{
		if (TManagedArray<bool>* const DirtyAttribute = GetManagedArrayCollection()->FindAttributeTyped<bool>(PropertyFacadeNames::DirtyName, PropertyFacadeNames::PropertyGlobalGroup);
			DirtyAttribute && DirtyAttribute->Num())
		{
			(*DirtyAttribute)[0] = true;
		}
	}

	void FCollectionPropertyFacade::EnableFlags(int32 KeyIndex, ECollectionPropertyFlags Flags, bool bEnable)
	{
		const ECollectionPropertyFlags CurrentFlags = (ECollectionPropertyFlags)FlagsArray[KeyIndex];

		if (bEnable)
		{
			if (!EnumHasAllFlags(CurrentFlags, Flags))
			{
				EnumAddFlags(ModifyFlagsArray()[KeyIndex], Flags | ECollectionPropertyFlags::Dirty);  // Changing any flags adds the dirty flag
			}
		}
		else
		{
			if (EnumHasAnyFlags(CurrentFlags, Flags))
			{
				TArrayView<ECollectionPropertyFlags> MutableFlagsArray = ModifyFlagsArray();
				EnumRemoveFlags(MutableFlagsArray[KeyIndex], Flags);
				EnumAddFlags(MutableFlagsArray[KeyIndex], ECollectionPropertyFlags::Dirty);  // Changing any flags adds the dirty flag, dirty flags can only be removed from the ClearDirtyFlags function
			}
		}
	}

	void FCollectionPropertyFacade::UpdateProperties(const TSharedPtr<const FManagedArrayCollection>& InManagedArrayCollection)
	{
		FCollectionPropertyConstFacade InPropertyFacade(InManagedArrayCollection);
		if (InPropertyFacade.IsValid())
		{
			const int32 NumInKeys = InPropertyFacade.Num();
			for (int32 InKeyIndex = 0; InKeyIndex < NumInKeys; ++InKeyIndex)
			{
				const FName& PropertyName = InPropertyFacade.GetKeyName(InKeyIndex);
				const int32 PropertyIndex = GetKeyNameIndex(PropertyName);
				if (PropertyIndex == INDEX_NONE)
				{
					continue;
				}
				SetFlags(PropertyIndex, InPropertyFacade.GetFlags(InKeyIndex));
				// Setting as FVector3f since that is the underlying type
				SetLowValue(PropertyIndex, InPropertyFacade.GetLowValue<FVector3f>(InKeyIndex));
				SetHighValue(PropertyIndex, InPropertyFacade.GetHighValue<FVector3f>(InKeyIndex));
				SetStringValue(PropertyIndex, InPropertyFacade.GetStringValue(InKeyIndex));
			}
		}
	}

	FCollectionPropertyMutableFacade::FCollectionPropertyMutableFacade(const TSharedPtr<FManagedArrayCollection>& InManagedArrayCollection)
		: FCollectionPropertyFacade(InManagedArrayCollection, NoInit)
	{
		UpdateArrays();
		RebuildKeyIndices();
	}

	void FCollectionPropertyMutableFacade::DefineSchema()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FCollectionPropertyMutableFacade_DefineSchema);
		GetManagedArrayCollection()->AddAttribute<FName>(PropertyFacadeNames::KeyNameName, PropertyFacadeNames::PropertyGroup);
		GetManagedArrayCollection()->AddAttribute<FVector3f>(PropertyFacadeNames::LowValueName, PropertyFacadeNames::PropertyGroup);
		GetManagedArrayCollection()->AddAttribute<FVector3f>(PropertyFacadeNames::HighValueName, PropertyFacadeNames::PropertyGroup);
		GetManagedArrayCollection()->AddAttribute<FString>(PropertyFacadeNames::StringValueName, PropertyFacadeNames::PropertyGroup);
		GetManagedArrayCollection()->AddAttribute<uint8>(PropertyFacadeNames::FlagsName, PropertyFacadeNames::PropertyGroup);

		// Add the global group and check whether it has one single element, otherwise add it and make the property collection dirty
		TManagedArray<bool>& DirtyAttribute = GetManagedArrayCollection()->AddAttribute<bool>(PropertyFacadeNames::DirtyName, PropertyFacadeNames::PropertyGlobalGroup);
		if (!GetManagedArrayCollection()->NumElements(PropertyFacadeNames::PropertyGlobalGroup))
		{
			GetManagedArrayCollection()->AddElements(1, PropertyFacadeNames::PropertyGlobalGroup);
			DirtyAttribute.Fill(true);
		}

		UpdateArrays();
		RebuildKeyIndices();
	}

	int32 FCollectionPropertyMutableFacade::AddProperty(const FName& Key, bool bEnabled, bool bAnimatable, bool bIntrinsic)
	{
		const ECollectionPropertyFlags Flags =
			(bEnabled ? ECollectionPropertyFlags::Enabled : ECollectionPropertyFlags::None) |
			(bAnimatable ? ECollectionPropertyFlags::Animatable : ECollectionPropertyFlags::None) |
			(bIntrinsic ? ECollectionPropertyFlags::Intrinsic : ECollectionPropertyFlags::None);
		return AddProperty(Key, Flags);
	}

	int32 FCollectionPropertyMutableFacade::AddProperty(const FName& Key, ECollectionPropertyFlags Flags)
	{
		const int32 Index = GetManagedArrayCollection()->AddElements(1, PropertyFacadeNames::PropertyGroup);

		// Update the arrayviews in case the new element triggered a reallocation 
		UpdateArrays();

		// Setup the new element's default value and enable the property by default
		ModifyKeyNameArray()[Index] = Key;
		ModifyLowValueArray()[Index] = ModifyHighValueArray()[Index] = FVector3f::ZeroVector;
		ModifyFlagsArray()[Index] = Flags;

		// Update search map
		KeyNameIndices.Emplace(KeyNameArray[Index], Index);

		return Index;
	}

	int32 FCollectionPropertyMutableFacade::AddProperties(const TArray<FName>& Keys, bool bEnabled, bool bAnimatable, bool bIntrinsic)
	{
		const ECollectionPropertyFlags Flags =
			(bEnabled ? ECollectionPropertyFlags::Enabled : ECollectionPropertyFlags::None) |
			(bAnimatable ? ECollectionPropertyFlags::Animatable : ECollectionPropertyFlags::None) |
			(bIntrinsic ? ECollectionPropertyFlags::Intrinsic : ECollectionPropertyFlags::None);

		return AddProperties(Keys, Flags);
	}

	int32 FCollectionPropertyMutableFacade::AddProperties(const TArray<FName>& Keys, ECollectionPropertyFlags Flags)
	{
		if (const int32 NumProperties = Keys.Num())
		{
			const int32 StartIndex = GetManagedArrayCollection()->AddElements(NumProperties, PropertyFacadeNames::PropertyGroup);

			// Update the arrayviews in case the new elements triggered a reallocation 
			UpdateArrays();

			for (int32 Index = StartIndex; Index < NumProperties + StartIndex; ++Index)
			{
				// Setup the new elements' default value and enable the property by default
				ModifyKeyNameArray()[Index] = Keys[Index - StartIndex];
				ModifyLowValueArray()[Index] = ModifyHighValueArray()[Index] = FVector3f::ZeroVector;
				ModifyFlagsArray()[Index] = Flags;

				// Update search map
				KeyNameIndices.Emplace(KeyNameArray[Index], Index);
			}

			return StartIndex;
		}
		return INDEX_NONE;
	}

	void FCollectionPropertyMutableFacade::Append(const TSharedPtr<const FManagedArrayCollection>& InManagedArrayCollection, bool bUpdateExistingProperties)
	{		
		Update(InManagedArrayCollection, ECollectionPropertyUpdateFlags::AppendNewProperties | (bUpdateExistingProperties ? ECollectionPropertyUpdateFlags::UpdateExistingProperties : ECollectionPropertyUpdateFlags::None));
	}

	void FCollectionPropertyMutableFacade::Copy(const FManagedArrayCollection& InManagedArrayCollection)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FCollectionPropertyMutableFacade_Copy);

		if (InManagedArrayCollection.HasAttribute(PropertyFacadeNames::KeyNameName, PropertyFacadeNames::PropertyGroup) &&
			InManagedArrayCollection.HasAttribute(PropertyFacadeNames::LowValueName, PropertyFacadeNames::PropertyGroup) &&
			InManagedArrayCollection.HasAttribute(PropertyFacadeNames::HighValueName, PropertyFacadeNames::PropertyGroup) &&
			InManagedArrayCollection.HasAttribute(PropertyFacadeNames::StringValueName, PropertyFacadeNames::PropertyGroup) &&
			InManagedArrayCollection.HasAttribute(PropertyFacadeNames::FlagsName, PropertyFacadeNames::PropertyGroup) &&
			InManagedArrayCollection.HasAttribute(PropertyFacadeNames::DirtyName, PropertyFacadeNames::PropertyGlobalGroup))
		{
			// Compare keys
			bool bKeyIndicesMatch = false;
			const TManagedArray<FName>* InKeyManagedArray = InManagedArrayCollection.FindAttributeTyped<FName>(PropertyFacadeNames::KeyNameName, PropertyFacadeNames::PropertyGroup);
			const TManagedArray<FName>* OrigKeyManagedArray = GetManagedArrayCollection()->FindAttributeTyped<FName>(PropertyFacadeNames::KeyNameName, PropertyFacadeNames::PropertyGroup);
			if (InKeyManagedArray && OrigKeyManagedArray && InKeyManagedArray->Num() == OrigKeyManagedArray->Num())
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FCollectionPropertyMutableFacade_Copy_CompareKeys);
				bKeyIndicesMatch = true;
				for (int32 Index = 0; Index < OrigKeyManagedArray->Num(); ++Index)
				{
					if ((*InKeyManagedArray)[Index] != (*OrigKeyManagedArray)[Index])
					{
						bKeyIndicesMatch = false;
						break;
					}
				}
			}
			if (!bKeyIndicesMatch)
			{
				GetManagedArrayCollection()->CopyAttribute(InManagedArrayCollection, PropertyFacadeNames::KeyNameName, PropertyFacadeNames::PropertyGroup);
			}
			GetManagedArrayCollection()->CopyAttribute(InManagedArrayCollection, PropertyFacadeNames::LowValueName, PropertyFacadeNames::PropertyGroup);
			GetManagedArrayCollection()->CopyAttribute(InManagedArrayCollection, PropertyFacadeNames::HighValueName, PropertyFacadeNames::PropertyGroup);
			GetManagedArrayCollection()->CopyAttribute(InManagedArrayCollection, PropertyFacadeNames::StringValueName, PropertyFacadeNames::PropertyGroup);
			GetManagedArrayCollection()->CopyAttribute(InManagedArrayCollection, PropertyFacadeNames::FlagsName, PropertyFacadeNames::PropertyGroup);
			UpdateArrays();
			if (!bKeyIndicesMatch)
			{
				RebuildKeyIndices();
			}
			GetManagedArrayCollection()->CopyAttribute(InManagedArrayCollection, PropertyFacadeNames::DirtyName, PropertyFacadeNames::PropertyGlobalGroup);
		}
		else
		{
			GetManagedArrayCollection()->Reset();
		}
	}

	void FCollectionPropertyMutableFacade::Update(const TSharedPtr<const FManagedArrayCollection>& InManagedArrayCollection, ECollectionPropertyUpdateFlags UpdateFlags)
	{
		if (UpdateFlags == ECollectionPropertyUpdateFlags::None)
		{
			// Nothing to do
			return;
		}
		if (UpdateFlags == ECollectionPropertyUpdateFlags::UpdateExistingProperties)
		{
			return UpdateProperties(InManagedArrayCollection);
		}

		const bool bAppendNewProperties = (UpdateFlags & ECollectionPropertyUpdateFlags::AppendNewProperties) != ECollectionPropertyUpdateFlags::None;
		const bool bUpdateExistingProperties = (UpdateFlags & ECollectionPropertyUpdateFlags::UpdateExistingProperties) != ECollectionPropertyUpdateFlags::None;
		const bool bRemoveMissingProperties = (UpdateFlags & ECollectionPropertyUpdateFlags::RemoveMissingProperties) != ECollectionPropertyUpdateFlags::None;
		const bool bDisableMissingProperties = (UpdateFlags & ECollectionPropertyUpdateFlags::DisableMissingProperties) != ECollectionPropertyUpdateFlags::None;

		FCollectionPropertyConstFacade InPropertyFacade(InManagedArrayCollection);
		if (InPropertyFacade.IsValid())
		{
			DefineSchema();

			const int32 PrevNumKeys = Num();

			if (bAppendNewProperties || bUpdateExistingProperties)
			{
				const int32 NumInKeys = InPropertyFacade.Num();
				for (int32 InKeyIndex = 0; InKeyIndex < NumInKeys; ++InKeyIndex)
				{
					const FName& PropertyName = InPropertyFacade.GetKeyName(InKeyIndex);
					const ECollectionPropertyFlags PropertyFlags = InPropertyFacade.GetFlags(InKeyIndex);
					int32 NewPropertyIndex = GetKeyNameIndex(PropertyName);
					if (NewPropertyIndex == INDEX_NONE)
					{
						if (bAppendNewProperties)
						{
							NewPropertyIndex = AddProperty(PropertyName, PropertyFlags);
						}
						else
						{
							continue;
						}
					}
					else if (!bUpdateExistingProperties)
					{
						continue;
					}
					else
					{
						SetFlags(NewPropertyIndex, PropertyFlags);
					}

					// Setting as FVector3f since that is the underlying type
					SetLowValue(NewPropertyIndex, InPropertyFacade.GetLowValue<FVector3f>(InKeyIndex));
					SetHighValue(NewPropertyIndex, InPropertyFacade.GetHighValue<FVector3f>(InKeyIndex));
					SetStringValue(NewPropertyIndex, InPropertyFacade.GetStringValue(InKeyIndex));
				}
			}
			if (bRemoveMissingProperties || bDisableMissingProperties)
			{
				TArray<int32> KeyIndicesToRemove;
				for (int32 ExistingKeyIndex = 0; ExistingKeyIndex < PrevNumKeys; ++ExistingKeyIndex)
				{
					if (InPropertyFacade.GetKeyNameIndex(GetKeyName(ExistingKeyIndex)) == INDEX_NONE)
					{
						if (bRemoveMissingProperties)
						{
							KeyIndicesToRemove.Add(ExistingKeyIndex);
						}
						else if(bDisableMissingProperties)
						{
							SetEnabled(ExistingKeyIndex, false);
						}
					}
				}

				if (!KeyIndicesToRemove.IsEmpty())
				{
					GetManagedArrayCollection()->RemoveElements(PropertyFacadeNames::PropertyGroup, KeyIndicesToRemove);
					UpdateArrays();
					RebuildKeyIndices();
				}
			}
		}
	}

	int32 FCollectionPropertyMutableFacade::AddWeightedFloatValue(const FName& Key, const FVector2f& Value, bool bEnabled, bool bAnimatable, bool bIntrinsic)
	{
		const int32 KeyIndex = AddProperty(Key, bEnabled, bAnimatable, bIntrinsic);
		SetWeightedFloatValue(KeyIndex, Value);
		return KeyIndex;
	}

	int32 FCollectionPropertyMutableFacade::AddWeightedFloatValue(const FName& Key, const FVector2f& Value, ECollectionPropertyFlags Flags)
	{
		const int32 KeyIndex = AddProperty(Key, Flags);
		SetWeightedFloatValue(KeyIndex, Value);
		return KeyIndex;
	}

	int32 FCollectionPropertyMutableFacade::AddStringValue(const FName& Key, const FString& Value, bool bEnabled, bool bAnimatable, bool bIntrinsic)
	{
		const int32 KeyIndex = AddProperty(Key, bEnabled, bAnimatable, bIntrinsic);
		SetStringValue(Key, Value);
		return KeyIndex;
	}

	int32 FCollectionPropertyMutableFacade::AddStringValue(const FName& Key, const FString& Value, ECollectionPropertyFlags Flags)
	{
		const int32 KeyIndex = AddProperty(Key, Flags);
		SetStringValue(Key, Value);
		return KeyIndex;
	}

	void FCollectionPropertyMutableFacade::PostSerialize(const FArchive& Ar)
	{
		if (Ar.IsLoading())
		{
			if (const TManagedArray<FString>* const InKeyArray = ManagedArrayCollection->FindAttributeTyped<FString>(PropertyFacadeNames::KeyName, PropertyFacadeNames::PropertyGroup))
			{
				if (KeyNameArray.IsEmpty())
				{
					// This is an old property facade using string rather than name-based keys. Transfer strings over to names.
					GetManagedArrayCollection()->AddAttribute<FName>(PropertyFacadeNames::KeyNameName, PropertyFacadeNames::PropertyGroup);
					UpdateArrays();
					check(InKeyArray->Num() == KeyNameArray.Num());
					for (int32 Index = 0; Index < KeyNameArray.Num(); ++Index)
					{
						ModifyKeyNameArray()[Index] = FName((*InKeyArray)[Index]);
					}
					RebuildKeyIndices();
				}

				GetManagedArrayCollection()->RemoveAttribute(PropertyFacadeNames::KeyName, PropertyFacadeNames::PropertyGroup);
			}
			// Add Dirty attribute if missing from a previously valid schema
			if (ManagedArrayCollection->HasAttribute(PropertyFacadeNames::KeyNameName, PropertyFacadeNames::PropertyGroup) &&
				ManagedArrayCollection->HasAttribute(PropertyFacadeNames::LowValueName, PropertyFacadeNames::PropertyGroup) &&
				ManagedArrayCollection->HasAttribute(PropertyFacadeNames::HighValueName, PropertyFacadeNames::PropertyGroup) &&
				ManagedArrayCollection->HasAttribute(PropertyFacadeNames::StringValueName, PropertyFacadeNames::PropertyGroup) &&
				ManagedArrayCollection->HasAttribute(PropertyFacadeNames::FlagsName, PropertyFacadeNames::PropertyGroup))
			{
				TManagedArray<bool>& DirtyAttribute = GetManagedArrayCollection()->AddAttribute<bool>(PropertyFacadeNames::DirtyName, PropertyFacadeNames::PropertyGlobalGroup);
				if (!GetManagedArrayCollection()->NumElements(PropertyFacadeNames::PropertyGlobalGroup))
				{
					GetManagedArrayCollection()->AddElements(1, PropertyFacadeNames::PropertyGlobalGroup);
					DirtyAttribute.Fill(true);
				}
			}
		}
	}
}  // End namespace Chaos::Softs
