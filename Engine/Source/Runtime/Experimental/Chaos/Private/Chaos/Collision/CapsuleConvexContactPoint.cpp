// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/Collision/CapsuleConvexContactPoint.h"
#include "Chaos/Collision/ContactPoint.h"
#include "Chaos/Capsule.h"
#include "Chaos/Convex.h"
#include "Chaos/GJK.h"
#include "Chaos/GJKShape.h"
#include "Chaos/ImplicitObjectScaled.h"

namespace Chaos
{
	template<typename T_CONVEX>
	FContactPoint CapsuleConvexContactPointImpl(const FImplicitCapsule3& Capsule, const FRigidTransform3& CapsuleTransform, const T_CONVEX& Convex, const FRigidTransform3& ConvexTransform)
	{
		FContactPoint ContactPoint;
		if (Convex.NumPlanes() > 0)
		{
			const FRigidTransform3 CapsuleToConvexTransform = CapsuleTransform.GetRelativeTransform(ConvexTransform);
			FVec3 PosConvex, PosCapsuleInConvex, NormalConvex;
			FReal Penetration;
			int32 VertexIndexA, VertexIndexB;

			// Run GJK to find separating distance if available
			// NOTE: Capsule is treated as a line (its core shape), Convex margin is ignored so we are using the outer non-shrunken hull.
			// @todo(chaos): use GJKDistance and SAT when that fails rather that GJK/EPA (but this requires an edge list in the convex)
			bool bHaveResult = GJKPenetration<true>(MakeGJKShape(Convex), MakeGJKCoreShape(Capsule), CapsuleToConvexTransform, Penetration, PosConvex, PosCapsuleInConvex, NormalConvex, VertexIndexA, VertexIndexB);

			// Build the contact point
			if (bHaveResult)
			{
				const FVec3 PosCapsule = CapsuleToConvexTransform.InverseTransformPosition(PosCapsuleInConvex);
				const FReal Phi = -Penetration;

				ContactPoint.ShapeContactPoints[0] = PosCapsule;
				ContactPoint.ShapeContactPoints[1] = PosConvex;
				ContactPoint.ShapeContactNormal = NormalConvex;
				ContactPoint.Phi = Phi;
			}
		}

		return ContactPoint;
	}

	FContactPoint CapsuleConvexContactPoint(const FImplicitCapsule3& Capsule, const FRigidTransform3& CapsuleTransform, const FImplicitObject& Object, const FRigidTransform3& ConvexTransform)
	{
		if (const TImplicitObjectInstanced<FImplicitConvex3>* InstancedConvex = Object.template GetObject<const TImplicitObjectInstanced<FImplicitConvex3>>())
		{
			return CapsuleConvexContactPointImpl(Capsule, CapsuleTransform, *InstancedConvex, ConvexTransform);
		}
		else if (const TImplicitObjectScaled<FImplicitConvex3>* ScaledConvex = Object.template GetObject<const TImplicitObjectScaled<FImplicitConvex3>>())
		{
			return CapsuleConvexContactPointImpl(Capsule, CapsuleTransform, *ScaledConvex, ConvexTransform);
		}
		else if (const FImplicitConvex3* Convex = Object.template GetObject<const FImplicitConvex3>())
		{
			return CapsuleConvexContactPointImpl(Capsule, CapsuleTransform, *Convex, ConvexTransform);
		}
		return FContactPoint();
	}

}
