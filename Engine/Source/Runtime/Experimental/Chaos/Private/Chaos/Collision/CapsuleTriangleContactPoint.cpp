// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/Collision/CapsuleTriangleContactPoint.h"
#include "Chaos/Capsule.h"
#include "Chaos/Collision/ContactPoint.h"
#include "Chaos/Triangle.h"

//UE_DISABLE_OPTIMIZATION

namespace Chaos
{
	void AddCapsuleTriangleParallelEdgeManifoldContacts(const FVec3& P0, const FVec3& P1, const FVec3& EdgeP0, const FVec3& EdgeP1, const FReal R, const FReal RejectDistanceSq, const FReal NormalToleranceSq, FContactPointManifold& OutContactPoints);

	// Return true if the value V is in the closed/inclusive range [RangeMin, RangeMax]
	// @todo(chaos): move to Utilities.h
	inline bool InRangeClosed(const FReal V, const FReal RangeMin, const FReal RangeMax)
	{
		return ((V >= RangeMin) & (V <= RangeMax)) != 0;
	}

	// Return true if the value V is in the open/exclusive range (RangeMin, RangeMax)
	// @todo(chaos): move to Utilities.h
	inline bool InRangeOpen(const FReal V, const FReal RangeMin, const FReal RangeMax)
	{
		return ((V > RangeMin) & (V < RangeMax)) != 0;
	}


	// Generate the contact manifold between a capsule and a triangle.
	//
	// Algorithm overview:
	// 
	// - Triangle is in Capsule space
	// 
	// - Generate planes for the triangle face and edges
	// 
	// - Special case: If the segment does not cross triangle edge prism and is exactly parallel to an edge
	//		- Project segment onto edge and select contact point(s)
	// 
	// - For each segment point
	//		- If point is within the edge prism
	//			- Add face contact (either on end cap or cylinder)
	// 
	// - For each edge
	//		- Add edge contact where the segment is closest to the edge
	//		- When the angle between segment and face is small, create a face contact instead
	//
	void ConstructCapsuleTriangleOneShotManifold2(const FImplicitCapsule3& Capsule, const FTriangle& Triangle, const FReal CullDistance, FContactPointManifold& OutContactPoints)
	{
		//
		// @todo(chaos): this function is a good SIMD candidate
		//

		// Capsule segment
		const FVec3 P0 = Capsule.GetX1f();
		const FVec3 P1 = Capsule.GetX2f();
		const FReal R = Capsule.GetRadiusf();
		const FVec3 Axis = Capsule.GetAxisf();
		const FReal L = Capsule.GetHeightf();
		const FReal RejectDistance = R + CullDistance;
		const FReal RejectDistanceSq = RejectDistance * RejectDistance;

		// Triangle
		const FVec3& V0 = Triangle.GetVertex(0);
		const FVec3& V1 = Triangle.GetVertex(1);
		const FVec3& V2 = Triangle.GetVertex(2);
		const FVec3 Centroid = Triangle.GetCentroid();

		// Tolerances
		// @todo(chaos): Everything below needs to be made size agnostic (i.e., all checks with a distance tolerance
		// need to cope with very larger or small segments and triangles). This will probably not be good enough...(add some tests)
		const FReal DistanceTolerance = FReal(1.e-5f * Capsule.GetHeightf());
		const FReal NormalTolerance = FReal(1.e-5);
		const FReal NormalToleranceSq = NormalTolerance * NormalTolerance;
		const FReal FaceContactSinAngleThreshold = FReal(0.34);	// ~Sin(20deg)

		// Face plane
		const FVec3& FaceP = Triangle.GetVertex(0);	// Use centroid?
		FVec3 FaceN = FVec3::CrossProduct(Triangle.GetVertex(1) - Triangle.GetVertex(0), Triangle.GetVertex(2) - Triangle.GetVertex(0));
		if (!FaceN.Normalize(NormalToleranceSq))
		{
			// Degenerate triangle
			return;
		}

		// Signed distance of capsule segment points to triangle face
		const FReal FaceD0 = FVec3::DotProduct(P0 - FaceP, FaceN);
		const FReal FaceD1 = FVec3::DotProduct(P1 - FaceP, FaceN);
		const bool bIsParallelFace = FMath::IsNearlyEqual(FaceD0, FaceD1, DistanceTolerance);

		// Reject if too far from tri plane (distance cull)
		if ((FaceD0 > RejectDistance) && (FaceD1 > RejectDistance))
		{
			// Separated from triangle
			return;
		}

		// Reject if the middle of the capsule is inside the face (single-sided collision)
		const FReal FaceDMid = FReal(0.5) * (FaceD0 + FaceD1);
		if (FaceDMid < -FReal(DistanceTolerance))
		{
			// Far inside triangle
			return;
		}

		// Edge plane normals and signed distances to each segment point
		FVec3 EdgeNs[3];
		FReal EdgeD0s[3];
		FReal EdgeD1s[3];

		EdgeNs[0] = FVec3::CrossProduct(V0 - V2, FaceN);
		EdgeNs[0].Normalize(NormalToleranceSq);
		EdgeD0s[0] = FVec3::DotProduct(P0 - V0, EdgeNs[0]);
		EdgeD1s[0] = FVec3::DotProduct(P1 - V0, EdgeNs[0]);
		if (((EdgeD0s[0] > RejectDistance) & (EdgeD1s[0] > RejectDistance)) != 0)
		{
			// Separated from triangle
			return;
		}

		EdgeNs[1] = FVec3::CrossProduct(V1 - V0, FaceN);
		EdgeNs[1].Normalize(NormalToleranceSq);
		EdgeD0s[1] = FVec3::DotProduct(P0 - V1, EdgeNs[1]);
		EdgeD1s[1] = FVec3::DotProduct(P1 - V1, EdgeNs[1]);
		if (((EdgeD0s[1] > RejectDistance) & (EdgeD1s[1] > RejectDistance)) != 0)
		{
			// Separated from triangle
			return;
		}

		EdgeNs[2] = FVec3::CrossProduct(V2 - V1, FaceN);
		EdgeNs[2].Normalize(NormalToleranceSq);
		EdgeD0s[2] = FVec3::DotProduct(P0 - V2, EdgeNs[2]);
		EdgeD1s[2] = FVec3::DotProduct(P1 - V2, EdgeNs[2]);
		if (((EdgeD0s[2] > RejectDistance) & (EdgeD1s[2] > RejectDistance)) != 0)
		{
			// Separated from triangle
			return;
		}

		// Cull based on signed distance to capsule segment
		// The edge-segment data is saved for used later on to generate contacts if necessary
		FVec3 EdgeSegmentDeltas[3];
		FVec3 EdgeEdgePs[3];
		FVec3 EdgeSegmentPs[3];
		FReal EdgeEdgeTs[3];
		FReal EdgeSegmentTs[3];
		FReal EdgeDistSqs[3];
		FReal EdgeDistSigns[3];
		FReal EdgeDotFace[3];
		int32 EdgeVertexIndex0 = 2;
		for (int32 EdgeIndex = 0; EdgeIndex < 3; ++EdgeIndex)
		{
			const int32 EdgeVertexIndex1 = EdgeIndex;
			const FVec3& EdgeP0 = Triangle.GetVertex(EdgeVertexIndex0);
			const FVec3& EdgeP1 = Triangle.GetVertex(EdgeVertexIndex1);
			EdgeVertexIndex0 = EdgeVertexIndex1;

			// Find the nearest point on the capsule segment to the edge segment
			FReal SegmentT, EdgeT;
			FVec3 SegmentP, EdgeP;
			Utilities::NearestPointsOnLineSegments(P0, P1, EdgeP0, EdgeP1, SegmentT, EdgeT, SegmentP, EdgeP);

			// Calculate the separation vector (from triangle to capsule)
			FVec3 SegmentEdgeN = SegmentP - EdgeP;
			FReal SegmentEdgeDistSign = FReal(1);
			const FReal SegmentEdgeDistSq = SegmentEdgeN.SizeSquared();

			// If the near point on the capsule axis is inside the triangle, fix the normal
			const FReal DotEdge = FVec3::DotProduct(SegmentEdgeN, EdgeNs[EdgeIndex]);
			if (DotEdge < FReal(-NormalTolerance))
			{
				SegmentEdgeN = -SegmentEdgeN;
				SegmentEdgeDistSign = FReal(-1);
			}
			const FReal DotFace = FVec3::DotProduct(SegmentEdgeN, FaceN);

			// If the near point on the capsule axis is outside the triangle check for cull distance
			if (SegmentEdgeDistSign > FReal(0))
			{
				// We generate contacts when separation is within cull distance
				// Treat CullDistance as zero when colliding with the underneath of a triangle
				FReal SeparationAxisCullDistance = RejectDistance;
				if (DotFace < -NormalTolerance)
				{
					SeparationAxisCullDistance = R;
				}

				if (SegmentEdgeDistSq > FMath::Square(SeparationAxisCullDistance))
				{
					return;
				}
			}

			EdgeSegmentDeltas[EdgeIndex] = SegmentEdgeN;
			EdgeEdgePs[EdgeIndex] = EdgeP;
			EdgeSegmentPs[EdgeIndex] = SegmentP;
			EdgeEdgeTs[EdgeIndex] = EdgeT;
			EdgeSegmentTs[EdgeIndex] = SegmentT;
			EdgeDistSqs[EdgeIndex] = SegmentEdgeDistSq;
			EdgeDistSigns[EdgeIndex] = SegmentEdgeDistSign;
			EdgeDotFace[EdgeIndex] = DotFace;
		}

		// Handle the case where the end cap(s) are inside all the edge planes. This should be fairly common unless the triangles are
		// much smaller than the capsule.
		// NOTE: Here we can only collide with the end-cap that is closest to the triangle (unless they are both exactly the same distance)
		// because the farthest segment point would generate a cylinder contact, not an end cap contact.
		bool bCollided0 = false;
		bool bCollided1 = false;
		const bool bInsideAll0 = ((EdgeD0s[0] <= DistanceTolerance) & (EdgeD0s[1] <= DistanceTolerance) & (EdgeD0s[2] <= DistanceTolerance)) != 0;
		const bool bInsideAll1 = ((EdgeD1s[0] <= DistanceTolerance) & (EdgeD1s[1] <= DistanceTolerance) & (EdgeD1s[2] <= DistanceTolerance)) != 0;
		if ((bInsideAll0 & (FaceD0 < RejectDistance) & (FaceD0 < FaceD1 + DistanceTolerance)) != 0)
		{
			FContactPoint& ContactPoint = OutContactPoints[OutContactPoints.AddUninitialized()];
			ContactPoint.ShapeContactPoints[0] = P0 - R * FaceN;
			ContactPoint.ShapeContactPoints[1] = P0 - FaceD0 * FaceN;
			ContactPoint.ShapeContactNormal = FaceN;
			ContactPoint.Phi = FaceD0 - R;
			ContactPoint.ContactType = EContactPointType::VertexPlane;
			ContactPoint.FaceIndex = INDEX_NONE;
			bCollided0 = true;
		}
		if ((bInsideAll1 & (FaceD1 < RejectDistance) & (FaceD1 < FaceD0 + DistanceTolerance)) != 0)
		{
			FContactPoint& ContactPoint = OutContactPoints[OutContactPoints.AddUninitialized()];
			ContactPoint.ShapeContactPoints[0] = P1 - R * FaceN;
			ContactPoint.ShapeContactPoints[1] = P1 - FaceD1 * FaceN;
			ContactPoint.ShapeContactNormal = FaceN;
			ContactPoint.Phi = FaceD1 - R;
			ContactPoint.ContactType = EContactPointType::VertexPlane;
			ContactPoint.FaceIndex = INDEX_NONE;
			bCollided1 = true;
		}

		// If we added both contacts above, we are (parallel to surface and both points inside edge planes)
		if (OutContactPoints.Num() == 2)
		{
			// Full manifold
			return;
		}

		// Handle the parallel segment-edge case by clipping the segment to the edge
		// NOTE: If we are parallel to an edge (and outside the edge prism) we can only collide with the one edge
		bool bIsParallelEdge[3];
		const bool bEqualEdgeDist0 = FMath::IsNearlyEqual(EdgeD0s[0], EdgeD1s[0], DistanceTolerance);
		const bool bEqualEdgeDist1 = FMath::IsNearlyEqual(EdgeD0s[1], EdgeD1s[1], DistanceTolerance);
		const bool bEqualEdgeDist2 = FMath::IsNearlyEqual(EdgeD0s[2], EdgeD1s[2], DistanceTolerance);
		bIsParallelEdge[0] = ((EdgeD0s[0] >= FReal(0)) & (EdgeD1s[0] >= FReal(0)) & bEqualEdgeDist0) != 0;
		bIsParallelEdge[1] = ((EdgeD0s[1] >= FReal(0)) & (EdgeD1s[1] >= FReal(0)) & bEqualEdgeDist1) != 0;
		bIsParallelEdge[2] = ((EdgeD0s[2] >= FReal(0)) & (EdgeD1s[2] >= FReal(0)) & bEqualEdgeDist2) != 0;
		EdgeVertexIndex0 = 2;
		if ((bIsParallelFace & (bIsParallelEdge[0] | bIsParallelEdge[1] | bIsParallelEdge[2])) != 0)
		{
			for (int32 EdgeIndex = 0; EdgeIndex < 3; ++EdgeIndex)
			{
				const int32 EdgeVertexIndex1 = EdgeIndex;
				const FVec3& EdgeP0 = Triangle.GetVertex(EdgeVertexIndex0);
				const FVec3& EdgeP1 = Triangle.GetVertex(EdgeVertexIndex1);
				EdgeVertexIndex0 = EdgeVertexIndex1;
				if (bIsParallelEdge[EdgeIndex])
				{
					AddCapsuleTriangleParallelEdgeManifoldContacts(P0, P1, EdgeP0, EdgeP1, R, RejectDistanceSq, NormalToleranceSq, OutContactPoints);
					return;
				}
			}
		}

		// Sine of angle between segment axis and the triangle face
		const FReal AxisDotNormal = FVec3::DotProduct(Axis, FaceN);
		const FReal SinAxisFaceAngle = AxisDotNormal;
		const bool bPreferFaceContact = (FMath::Abs(SinAxisFaceAngle) < FaceContactSinAngleThreshold);

		// Generate contacts for the cylinder ends near the face. Only consider the ends where we did not generate an end-cap contact, 
		// and only when we are at a low angle to the face. 
		// The cylinder points can only be inside the edge planes if the segment points are within Radius of the edge planes
		const bool bNearAll0 = ((EdgeD0s[0] <= R + DistanceTolerance) & (EdgeD0s[1] <= R + DistanceTolerance) & (EdgeD0s[2] <= R + DistanceTolerance)) != 0;
		const bool bNearAll1 = ((EdgeD1s[0] <= R + DistanceTolerance) & (EdgeD1s[1] <= R + DistanceTolerance) & (EdgeD1s[2] <= R + DistanceTolerance)) != 0;
		const bool bCheckCylinder0 = ((!bCollided0) & bPreferFaceContact & bNearAll0) != 0;
		const bool bCheckCylinder1 = ((!bCollided1) & bPreferFaceContact & bNearAll1) != 0;
		if ((bCheckCylinder0 | bCheckCylinder1) != 0)
		{
			FVec3 RadialAxis = FVec3::CrossProduct(FVec3::CrossProduct(Axis, FaceN), Axis);
			if (RadialAxis.Normalize(NormalTolerance))
			{
				// We want Radial axis to point against the normal
				if (FVec3::DotProduct(RadialAxis, FaceN) > FReal(0))
				{
					RadialAxis = -RadialAxis;
				}

				// Utility to add a cylinder contact point, if it is within the edge planes
				const auto& TryAddCylinderContact = [R, &RadialAxis, &V0, &V1, &V2, &FaceN, &FaceP, &EdgeNs, DistanceTolerance, &OutContactPoints](const FVec3& P) -> void
				{
					const FVec3 CylinderP = P + R * RadialAxis;
					const FReal CylinderEdgeD0 = FVec3::DotProduct(CylinderP - V0, EdgeNs[0]);
					const FReal CylinderEdgeD1 = FVec3::DotProduct(CylinderP - V1, EdgeNs[1]);
					const FReal CylinderEdgeD2 = FVec3::DotProduct(CylinderP - V2, EdgeNs[2]);
					const bool bCylinderInsideAll = ((CylinderEdgeD0 <= DistanceTolerance) & (CylinderEdgeD1 <= DistanceTolerance) & (CylinderEdgeD2 <= DistanceTolerance)) != 0;
					if (bCylinderInsideAll)
					{
						const FReal CylinderFaceD = FVec3::DotProduct(CylinderP - FaceP, FaceN);

						FContactPoint& ContactPoint = OutContactPoints[OutContactPoints.AddUninitialized()];
						ContactPoint.ShapeContactPoints[0] = CylinderP;
						ContactPoint.ShapeContactPoints[1] = CylinderP - CylinderFaceD * FaceN;
						ContactPoint.ShapeContactNormal = FaceN;
						ContactPoint.Phi = CylinderFaceD;
						ContactPoint.ContactType = EContactPointType::VertexPlane;
						ContactPoint.FaceIndex = INDEX_NONE;
					}
				};

				if (bCheckCylinder0)
				{
					TryAddCylinderContact(P0);
				}
				if (bCheckCylinder1)
				{
					TryAddCylinderContact(P1);
				}
			}
		}

		// If we have a contact at both ends now, we are done
		if (OutContactPoints.Num() == 2)
		{
			// Full manifold
			return;
		}

		// Add edge contacts to the manifold
		EdgeVertexIndex0 = 2;
		for (int32 EdgeIndex = 0; EdgeIndex < 3; ++EdgeIndex)
		{
			const int32 EdgeVertexIndex1 = EdgeIndex;
			const FVec3& EdgeP0 = Triangle.GetVertex(EdgeVertexIndex0);
			const FVec3& EdgeP1 = Triangle.GetVertex(EdgeVertexIndex1);
			EdgeVertexIndex0 = EdgeVertexIndex1;

			// Reuse edge-segment data calculated in cull check above
			FVec3 SegmentEdgeN = EdgeSegmentDeltas[EdgeIndex];
			const FVec3& EdgeP = EdgeEdgePs[EdgeIndex];
			const FVec3& SegmentP = EdgeSegmentPs[EdgeIndex];
			const FReal EdgeT = EdgeEdgeTs[EdgeIndex];
			const FReal SegmentT = EdgeSegmentTs[EdgeIndex];
			const FReal SegmentEdgeDistSq = EdgeDistSqs[EdgeIndex];
			const FReal SegmentEdgeDistSign = EdgeDistSigns[EdgeIndex];
			const FReal DotFace = EdgeDotFace[EdgeIndex];

			// We only care about edges if at least one capsule segment point is outside the edge plane
			// (internal points were handled already)
			if (((EdgeD0s[EdgeIndex] > -DistanceTolerance) | (EdgeD1s[EdgeIndex] > -DistanceTolerance)) != 0)
			{
				// Don't collide with inside face
				if (DotFace < FReal(-NormalTolerance))
				{
					continue;
				}

				// We will create a face contact rather than an edge contact where possible.
				// When the angle between the axis and the face is below a threshold.
				const bool bInEdgeRange = InRangeOpen(EdgeT, FReal(0), FReal(1));
				const bool bInSegmentRange = InRangeOpen(SegmentT, FReal(0), FReal(1));
				const bool bCrossedEdgeSegment = ((bInEdgeRange & bInSegmentRange) != 0);

				// Calculate separation distance and normal
				// If we have zero separation, we cannot renormalize the separation vector so we must calculate the normal
				FReal SegmentEdgeDist;
				if (SegmentEdgeDistSq > NormalToleranceSq)
				{
					// Get the signed distance and separating axis
					SegmentEdgeDist = FMath::Sqrt(SegmentEdgeDistSq);
					SegmentEdgeN = SegmentEdgeN / SegmentEdgeDist;
					SegmentEdgeDist *= SegmentEdgeDistSign;
				}
				else
				{
					// Segment passes right through edge - calculate normal
					SegmentEdgeDist = FReal(0);
					SegmentEdgeN = FVec3::CrossProduct(Axis, EdgeP1 - EdgeP0);
					if (!SegmentEdgeN.Normalize(NormalToleranceSq))
					{
						continue;
					}
					if (FVec3::DotProduct(SegmentEdgeN, FaceN) < FReal(0))
					{
						SegmentEdgeN = -SegmentEdgeN;
					}
					const FReal DotEdge = FVec3::DotProduct(EdgeNs[EdgeIndex], SegmentEdgeN);
					if (DotEdge < -NormalTolerance)
					{
						continue;
					}
				}

				if ((bCrossedEdgeSegment & bPreferFaceContact) == 0)
				{
					// We cannot collide with the inside of the edge
					const FReal DotEdge = FVec3::DotProduct(EdgeNs[EdgeIndex], SegmentEdgeN);
					if (DotEdge < -NormalTolerance)
					{
						continue;
					}

					// For Vertex contacts, check that the normal is in the valid range
					// it must point away from the edge vectors that share the vertex
					if (EdgeT == FReal(0))
					{
						const int32 PrevEdgeVertexIndex0 = (EdgeIndex >= 2) ? (EdgeIndex - 2) : (EdgeIndex - 2 + 3);
						const FVec3& PrevEdgeP0 = Triangle.GetVertex(PrevEdgeVertexIndex0);
						const FVec3& PrevEdgeP1 = EdgeP0;

						// NOTE: both edge vetors here point towards the shared vertex (at EdgeP0)
						const FReal PrevEdgeDotNormal = FVec3::DotProduct(PrevEdgeP1 - PrevEdgeP0, SegmentEdgeN);
						const FReal EdgeDotNormal = FVec3::DotProduct(EdgeP0 - EdgeP1, SegmentEdgeN);
						if ((PrevEdgeDotNormal < -NormalTolerance) || (EdgeDotNormal < -NormalTolerance))
						{
							continue;
						}
					}
					if (EdgeT == FReal(1))
					{
						const int32 NextEdgeVertexIndex1 = (EdgeIndex < 2) ? EdgeIndex + 1 : 0;
						const FVec3& NextEdgeP0 = EdgeP1;
						const FVec3& NextEdgeP1 = Triangle.GetVertex(NextEdgeVertexIndex1);

						// NOTE: both edge vetors here point towards the shared vertex (at EgdeP1)
						const FReal EdgeDotNormal = FVec3::DotProduct(EdgeP1 - EdgeP0, SegmentEdgeN);
						const FReal NextEdgeDotNormal = FVec3::DotProduct(NextEdgeP0 - NextEdgeP1, SegmentEdgeN);
						if ((EdgeDotNormal < -NormalTolerance) || (NextEdgeDotNormal < -NormalTolerance))
						{
							continue;
						}
					}
				}

				// If we are within the face angle tolerance, generate a face contact rather than an edge one
				// (but only if we have an edge contact and not a vertex one)
				// NOTE: we rely on the fact that the Ts will be exactly 0 or 1 when the near point is outside the segment or edge
				if ((bCrossedEdgeSegment & bPreferFaceContact) != 0)
				{
					const FVec3 CapsuleP = SegmentP - R * SegmentEdgeN;
					const FReal CapsuleDist = FVec3::DotProduct(CapsuleP - FaceP, FaceN);

					FContactPoint& ContactPoint = OutContactPoints[OutContactPoints.AddUninitialized()];
					ContactPoint.ShapeContactPoints[0] = CapsuleP;
					ContactPoint.ShapeContactPoints[1] = CapsuleP - CapsuleDist * FaceN;
					ContactPoint.ShapeContactNormal = FaceN;
					ContactPoint.Phi = CapsuleDist;
					ContactPoint.ContactType = EContactPointType::VertexPlane;
					ContactPoint.FaceIndex = INDEX_NONE;
				}
				else
				{
					FContactPoint& ContactPoint = OutContactPoints[OutContactPoints.AddUninitialized()];
					ContactPoint.ShapeContactPoints[0] = SegmentP - R * SegmentEdgeN;
					ContactPoint.ShapeContactPoints[1] = EdgeP;
					ContactPoint.ShapeContactNormal = SegmentEdgeN;
					ContactPoint.Phi = SegmentEdgeDist - R;
					ContactPoint.ContactType = EContactPointType::EdgeEdge;
					ContactPoint.FaceIndex = INDEX_NONE;
				}
			}
		}
	}

	// Utility for ConstructCapsuleTriangleOneShotManifold2 to handle the rare case when a capsule is perfectly aligned with an edge of a triangle.
	// In this case we project the segment onto the edge and, depending on whether there is any overlap, use either the closest point or the
	// two clipped points as contacts.
	void AddCapsuleTriangleParallelEdgeManifoldContacts(const FVec3& P0, const FVec3& P1, const FVec3& EdgeP0, const FVec3& EdgeP1, const FReal R, const FReal RejectDistanceSq, const FReal NormalToleranceSq, FContactPointManifold& OutContactPoints)
	{
		// Utility to add a contact to the array if it is within cull distance
		const auto& AddContact = [](const FVec3& SegmentEdgeC, const FVec3& SegmentEdgeDelta, const FReal R, const FReal RejectDistanceSq, const FReal NormalToleranceSq, FContactPointManifold& OutContactPoints) -> void
		{
			const FReal SegmentEdgeDistSq = SegmentEdgeDelta.SizeSquared();
			if ((SegmentEdgeDistSq < RejectDistanceSq) && (SegmentEdgeDistSq > NormalToleranceSq))
			{
				const FReal SegmentEdgeDist = FMath::Sqrt(SegmentEdgeDistSq);
				const FVec3 SegmentEdgeN = SegmentEdgeDelta / SegmentEdgeDist;

				FContactPoint& ContactPoint = OutContactPoints[OutContactPoints.AddUninitialized()];
				ContactPoint.ShapeContactPoints[0] = SegmentEdgeC + (SegmentEdgeDist - R) * SegmentEdgeN;
				ContactPoint.ShapeContactPoints[1] = SegmentEdgeC;
				ContactPoint.ShapeContactNormal = SegmentEdgeN;
				ContactPoint.Phi = SegmentEdgeDist - R;
				ContactPoint.ContactType = EContactPointType::EdgeEdge;
				ContactPoint.FaceIndex = INDEX_NONE;
			}
		};

		const FVec3 EdgeDelta = EdgeP1 - EdgeP0;
		const FReal EdgeLenSq = EdgeDelta.SizeSquared();
		if (EdgeLenSq > NormalToleranceSq)
		{
			// Project the first segment point onto the edge
			const FReal T0 = FVec3::DotProduct(P0 - EdgeP0, EdgeDelta) / EdgeLenSq;
			const FReal T1 = FVec3::DotProduct(P1 - EdgeP0, EdgeDelta) / EdgeLenSq;

			// If both points have T < 0, only use the point with highest T
			// If both points have T > 1, only use the point with lowest T
			// Otherwise use both points with T clamped to [0,1]
			const bool bInRange0 = InRangeClosed(T0, FReal(0), FReal(1));
			const bool bInRange1 = InRangeClosed(T1, FReal(0), FReal(1));
			if ((bInRange0 | bInRange1) != 0)
			{
				// At least one segment point projects to a point inside the edge extents.
				// Clip the edge-mapped segement and use the clipped verts as contacts.

				const FVec3 SegmentEdgeDelta0 = P0 - (EdgeP0 + T0 * EdgeDelta);
				const FVec3 SegmentEdgeC0 = EdgeP0 + FMath::Clamp(T0, FReal(0), FReal(1)) * EdgeDelta;
				AddContact(SegmentEdgeC0, SegmentEdgeDelta0, R, RejectDistanceSq, NormalToleranceSq, OutContactPoints);

				const FVec3 SegmentEdgeDelta1 = P1 - (EdgeP0 + T1 * EdgeDelta);
				const FVec3 SegmentEdgeC1 = EdgeP0 + FMath::Clamp(T1, FReal(0), FReal(1)) * EdgeDelta;
				AddContact(SegmentEdgeC1, SegmentEdgeDelta1, R, RejectDistanceSq, NormalToleranceSq, OutContactPoints);
			}
			else
			{
				// Both segment points projected to the edge are outside the edge extents.
				// Use the segment point nearest the edge as the contact.

				FReal SegmentEdgeT;
				FVec3 SegmentEdgeP;
				if ((T0 < FReal(0)) && (T1 < FReal(0)))
				{
					SegmentEdgeT = (T0 > T1) ? T0 : T1;
					SegmentEdgeP = (T0 > T1) ? P0 : P1;
				}
				else
				{
					SegmentEdgeT = (T0 < T1) ? T0 : T1;
					SegmentEdgeP = (T0 < T1) ? P0 : P1;
				}
				const FVec3 SegmentEdgeC = EdgeP0 + FMath::Clamp(SegmentEdgeT, FReal(0), FReal(1)) * EdgeDelta;
				const FVec3 SegmentEdgeDelta = (SegmentEdgeP - SegmentEdgeC);
				AddContact(SegmentEdgeC, SegmentEdgeDelta, R, RejectDistanceSq, NormalToleranceSq, OutContactPoints);
			}
		}
	}

}