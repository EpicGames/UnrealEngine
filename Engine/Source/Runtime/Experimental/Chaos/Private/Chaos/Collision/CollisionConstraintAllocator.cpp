// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/Collision/CollisionConstraintAllocator.h"
#include "Chaos/PBDCollisionConstraints.h"
#include "Tasks/Task.h"

namespace Chaos
{

	namespace Private
	{
		FPBDCollisionConstraintPtr FCollisionContextAllocator::CreateConstraint(
			FGeometryParticleHandle* Particle0,
			const FImplicitObject* Implicit0,
			const FPerShapeData* Shape0,
			const FBVHParticles* Simplicial0,
			const FRigidTransform3& ShapeRelativeTransform0,
			FGeometryParticleHandle* Particle1,
			const FImplicitObject* Implicit1,
			const FPerShapeData* Shape1,
			const FBVHParticles* Simplicial1,
			const FRigidTransform3& ShapeRelativeTransform1,
			const FReal CullDistance,
			const bool bUseManifold,
			const EContactShapesType ShapePairType)
		{
			FPBDCollisionConstraintPtr Constraint = CreateConstraint();

			if (Constraint.IsValid())
			{
				Constraint->SetContainer(CollisionContainer);
				FPBDCollisionConstraint::Make(Particle0, Implicit0, Shape0, Simplicial0, ShapeRelativeTransform0, Particle1, Implicit1, Shape1, Simplicial1, ShapeRelativeTransform1, CullDistance, bUseManifold, ShapePairType, *Constraint);
			}

			return Constraint;
		}

		void FCollisionConstraintAllocator::ProcessNewMidPhases()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_Collisions_ProcessNewMidPhases);

			FMemMark Mark(FMemStack::Get());
			TArray<FParticlePairMidPhase*, TMemStackAllocator<>> NewMidPhases;

			// Count the midphases so we can initialize the array
			int32 NumNewMidPhases = 0;
			for (TUniquePtr<FCollisionContextAllocator>& ContextAllocator : ContextAllocators)
			{
				NumNewMidPhases += ContextAllocator->NewMidPhases.Num();
			}
			NewMidPhases.Reserve(NumNewMidPhases);

			// Collect the midphases
			for (TUniquePtr<FCollisionContextAllocator>& ContextAllocator : ContextAllocators)
			{
				NewMidPhases.Append(ContextAllocator->NewMidPhases);
				ContextAllocator->NewMidPhases.Reset();
			}
			check(NewMidPhases.Num() == NumNewMidPhases);

			// For deterministic behaviour we need to sort the midphases so that, when they are added to 
			// our and each particle's lists, they are in a repeatable order
			// @todo(chaos): we could sort each context's array and then process them in order here instead
			if (bIsDeteministic)
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_Collisions_SortMidPhases);

				NewMidPhases.Sort(
					[](const FParticlePairMidPhase& L, const FParticlePairMidPhase& R)
					{
						return L.GetKey() < R.GetKey();
					});
			}

			// Register the midphases with each of their particles and add the midphase to the central list
			for (FParticlePairMidPhase* MidPhase : NewMidPhases)
			{
				FGeometryParticleHandle* Particle0 = MidPhase->GetParticle0();
				FGeometryParticleHandle* Particle1 = MidPhase->GetParticle1();

				// NOTE: It may look weird that we are passing the particle's self to the AddMidPhase method. 
				// The midphase represents two particles and it needs to know which one we're currently adding it to.
				// We could make this API nicer but providing a helper on the Particle, but it's a bit awkward to do so
				// and since AddMidPahse is only called from here we'll leave it like this for now.
				Particle0->ParticleCollisions().AddMidPhase(Particle0, MidPhase);
				Particle1->ParticleCollisions().AddMidPhase(Particle1, MidPhase);

#if CHAOS_MIDPHASE_OBJECTPOOL_ENABLED 
				AddMidPhase(FParticlePairMidPhasePtr(MidPhase, FParticlePairMidPhaseDeleter(MidPhasePool)));
#else
				AddMidPhase(FParticlePairMidPhasePtr(MidPhase));
#endif
			}
		}

		void FCollisionConstraintAllocator::ProcessNewNonCcdConstraints(const TArray<FPBDCollisionConstraint*>& NewConstraints, TArray<FPBDCollisionConstraint*>& AllCCDConstraints)
		{
			const int32 NumConstraints = NewConstraints.Num();

			if (Chaos::ShouldExecuteTasks(NumConstraints))
			{
				const int32 NumTasks = FMath::Max(FMath::Min(FTaskGraphInterface::Get().GetNumWorkerThreads(), Chaos::MaxNumWorkers), 1);
				const int32 ConstraintByTask = FMath::Max(FMath::DivideAndRoundUp(NumConstraints, NumTasks), CollisionSmallBatchSize);
				const int32 NumBatches = FMath::DivideAndRoundUp(NumConstraints, ConstraintByTask);

				TArray<UE::Tasks::FTask> PendingTasks;
				PendingTasks.Reserve(NumBatches);

				TArray<TArray<FPBDCollisionConstraint*>> CCDConstraintsBatch;
				CCDConstraintsBatch.SetNum(NumBatches);

				for (int32 BatchIndex = 0; BatchIndex < NumBatches; BatchIndex++)
				{
					const int32 StartIndex = BatchIndex * ConstraintByTask;
					int32 EndIndex = (BatchIndex + 1) * ConstraintByTask;
					EndIndex = FMath::Min(NumConstraints, EndIndex);

					TArray<FPBDCollisionConstraint*>& CCDConstraints = CCDConstraintsBatch[BatchIndex];

					UE::Tasks::FTask PendingTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, StartIndex, EndIndex, &NewConstraints, &CCDConstraints]
					{
						QUICK_SCOPE_CYCLE_COUNTER(STAT_Collisions_SubProcessNewConstraints);
						for (int32 Index = StartIndex; Index < EndIndex; Index++)
						{
							FPBDCollisionConstraint* Constraint = NewConstraints[Index];
							SetConstraintCookie(Constraint, Index);
							// If the constraint uses CCD, keep it in another list so we don't have to search the full list for CCD contacts
							if (Constraint->GetCCDEnabled())
							{
								CCDConstraints.Add(Constraint);
							}
						}
					}, UE::Tasks::ETaskPriority::High);
					PendingTasks.Add(PendingTask);
				}
				UE::Tasks::Wait(PendingTasks);

				// Single Threaded and protect the insertion order for determinism 
				for (int32 BatchIndex = 0; BatchIndex < CCDConstraintsBatch.Num(); BatchIndex++)
				{
					AllCCDConstraints.Append(CCDConstraintsBatch[BatchIndex]);
				}
			}
			else
			{
				// Small enough task to run immediately and avoid parallel overhead
				QUICK_SCOPE_CYCLE_COUNTER(STAT_Collisions_SubProcessNewConstraints);
				for (int32 Index = 0; Index < NumConstraints; Index++)
				{
					FPBDCollisionConstraint* Constraint = NewConstraints[Index];
					SetConstraintCookie(Constraint, Index);
					// If the constraint uses CCD, keep it in another list so we don't have to search the full list for CCD contacts
					if (Constraint->GetCCDEnabled())
					{
						AllCCDConstraints.Add(Constraint);
					}
				}
			}
		}

		void FCollisionConstraintAllocator::ProcessNewCcdConstraints()
		{

			
			const int32 NumConstraints = ActiveCCDConstraints.Num();
			

			if (Chaos::ShouldExecuteTasks(NumConstraints))
			{
				const int32 NumTasks = FMath::Max(FMath::Min(FTaskGraphInterface::Get().GetNumWorkerThreads(), Chaos::MaxNumWorkers), 1);
				const int32 ConstraintByTask = FMath::Max(FMath::DivideAndRoundUp(NumConstraints, NumTasks), CollisionSmallBatchSize);
				const int32 NumBatches = FMath::DivideAndRoundUp(NumConstraints, ConstraintByTask);

				TArray<UE::Tasks::FTask> PendingTasks;
				PendingTasks.Reserve(NumBatches);

				for (int32 BatchIndex = 0; BatchIndex < NumBatches; BatchIndex++)
				{
					const int32 StartIndex = BatchIndex * ConstraintByTask;
					int32 EndIndex = (BatchIndex + 1) * ConstraintByTask;
					EndIndex = BatchIndex == NumBatches - 1 ? FMath::Min(NumConstraints, EndIndex) : EndIndex;

					UE::Tasks::FTask PendingTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, StartIndex, EndIndex]
					{
						QUICK_SCOPE_CYCLE_COUNTER(STAT_Collisions_SubCCDProcessNewConstraints);
						for (int32 Index = StartIndex; Index < EndIndex; Index++)
						{
							FPBDCollisionConstraint* Constraint = ActiveCCDConstraints[Index];
							Constraint->GetContainerCookie().CCDConstraintIndex = Index;
						}
					}, UE::Tasks::ETaskPriority::High);
					PendingTasks.Add(PendingTask);
				}
				ContextAllocators[0]->NewActiveConstraints.Reset();
				UE::Tasks::Wait(PendingTasks);
			}
			else
			{
				// Small enough task to run immediately and avoid parallel overhead
				QUICK_SCOPE_CYCLE_COUNTER(STAT_Collisions_SubCCDProcessNewConstraints);
				for (int32 Index = 0; Index < NumConstraints; Index++)
				{
					FPBDCollisionConstraint* Constraint = ActiveCCDConstraints[Index];
					Constraint->GetContainerCookie().CCDConstraintIndex = Index;
				}
				ContextAllocators[0]->NewActiveConstraints.Reset();
			}
		}

		void FCollisionConstraintAllocator::ProcessNewConstraints()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_Collisions_ProcessNewConstraints);
			FMemMark Mark(FMemStack::Get());

			if (!ContextAllocators.IsEmpty())
			{
				const TArray<FPBDCollisionConstraint*>& NewConstraints = ContextAllocators[0]->NewActiveConstraints;
				ActiveConstraints.Append(NewConstraints);

				ProcessNewNonCcdConstraints(NewConstraints, ActiveCCDConstraints);
				ProcessNewCcdConstraints();
			}
		}

		void FCollisionConstraintAllocator::ProcessNewItems()
		{

			
			const int32 NumConstraints = ContextAllocators.IsEmpty() ? 0 : ContextAllocators[0]->NewActiveConstraints.Num();
			
			if (Chaos::ShouldExecuteTasks(NumConstraints))
			{
				const int32 NumTasks = FMath::Max(FMath::Min(FTaskGraphInterface::Get().GetNumWorkerThreads(), Chaos::MaxNumWorkers), 1);
				const int32 ConstraintByTask = FMath::Max(FMath::DivideAndRoundUp(NumConstraints, NumTasks), CollisionSmallBatchSize);
				const int32 NumBatches = FMath::DivideAndRoundUp(NumConstraints, ConstraintByTask);

				UE::Tasks::FTask PendingTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [this] 
				{
					ProcessNewMidPhases();
				}, UE::Tasks::ETaskPriority::High);

				ProcessNewConstraints();
				UE::Tasks::Wait(PendingTask);
			}
			else
			{
				ProcessNewMidPhases();
				ProcessNewConstraints();
			}
		}

		void FCollisionConstraintAllocator::EndDetectCollisions()
		{
			check(bInCollisionDetectionPhase);

			bInCollisionDetectionPhase = false;

			ProcessNewItems();
		}

		void FCollisionConstraintAllocator::ResetActiveConstraints()
		{
			// Disable existing constraints so that if they are not re-activated this tick
			// they do not have state indicating that that are still active.
			// @todo(chaos): ideally we would do this only for constraints that do not get reused this tick in EndDetectCollisions
			for (FPBDCollisionConstraint* Constraint : ActiveConstraints)
			{
				if (Constraint != nullptr)
				{
					Constraint->BeginTick();
				}
			}
		}

		void FCollisionConstraintAllocator::PruneExpiredMidPhases()
		{
			// NOTE: Called from the physics thread, and never from a physics task/parallel-for. No need for locks.

			// Determine which particle pairs are no longer overlapping
			// Prune all pairs which were not updated this tick as part of the collision
			// detection loop and are not asleep
			for (int32 Index = ParticlePairMidPhases.Num() - 1; Index >= 0; --Index)
			{
				FParticlePairMidPhasePtr& MidPhase = ParticlePairMidPhases[Index];

				// We could also add !MidPhase->IsInConstraintGraph() here, but we know that we will not be in the graph if we were
				// not active this tick and were not asleep. The constraint graph ejects all non-sleeping constraints each tick.
				// (There is a check in the collision destructor that verified this).
				if (!MidPhase->IsUsedSince(CurrentEpoch) && !MidPhase->IsSleeping())
				{
					// Remove from the particles' lists of contacts
					DetachParticlePairMidPhase(MidPhase.Get());

					// Destroy the midphase and its collisions
					MidPhase.Reset();

					// Remove the midphase from the list. 
					// ParticlePairMidPhases can get large so we allow it to shrink from time to time
					const int32 MaxSlack = 1000;
					const int32 Slack = ParticlePairMidPhases.Max() - ParticlePairMidPhases.Num();
					const bool bAllowShrink = (Slack > MaxSlack);
					ParticlePairMidPhases.RemoveAtSwap(Index, 1, bAllowShrink ? EAllowShrinking::Yes : EAllowShrinking::No);
				}
			}
		}

		void FCollisionConstraintAllocator::AddActiveConstraint(FPBDCollisionConstraint* Constraint)
		{
			// Add the constraint to the active list, but only if it hasn't been activated this epoch already
			FPBDCollisionConstraintContainerCookie& Cookie = Constraint->GetContainerCookie();
			if (Cookie.LastUsedEpoch != CurrentEpoch)
			{
				AddActiveConstraintImpl(Constraint);
			}
		}

		void FCollisionConstraintAllocator::AddActiveConstraintImpl(FPBDCollisionConstraint* Constraint)
		{
			FPBDCollisionConstraintContainerCookie& Cookie = Constraint->GetContainerCookie();

			Cookie.LastUsedEpoch = CurrentEpoch;
			Cookie.ConstraintIndex = INDEX_NONE;
			Cookie.CCDConstraintIndex = INDEX_NONE;

			// Add to the active list and remember the index for O(1) removal
			checkSlow(ActiveConstraints.Find(Constraint) == INDEX_NONE);
			Cookie.ConstraintIndex = ActiveConstraints.Add(Constraint);

			// If the constraint uses CCD, keep it in another list so we don't have to search the full list for CCD contacts
			if (Constraint->GetCCDEnabled())
			{
				checkSlow(ActiveCCDConstraints.Find(Constraint) == INDEX_NONE);
				Cookie.CCDConstraintIndex = ActiveCCDConstraints.Add(Constraint);
			}
		}

		void FCollisionConstraintAllocator::SetConstraintCookie(FPBDCollisionConstraint* Constraint, int32 Index)
		{
			FPBDCollisionConstraintContainerCookie& Cookie = Constraint->GetContainerCookie();

			Cookie.LastUsedEpoch = CurrentEpoch;
			Cookie.CCDConstraintIndex = INDEX_NONE;

			// Add to the active list and remember the index for O(1) removal
			Cookie.ConstraintIndex = Index;
		}

		void FCollisionConstraintAllocator::RemoveActiveConstraint(FPBDCollisionConstraint& Constraint)
		{
			FPBDCollisionConstraintContainerCookie& Cookie = Constraint.GetContainerCookie();

			// ConstraintIndex is only valid for one frame, so make sure we
			// were actually activated during the most recent tick
			if (Cookie.LastUsedEpoch != CurrentEpoch)
			{
				return;
			}

			// Remove from the active list
			if (Cookie.ConstraintIndex != INDEX_NONE)
			{
				check(ActiveConstraints[Cookie.ConstraintIndex] == &Constraint);
				ActiveConstraints[Cookie.ConstraintIndex] = nullptr;
				Cookie.ConstraintIndex = INDEX_NONE;
			}

			// Remove from the active CCD list
			if (Cookie.CCDConstraintIndex != INDEX_NONE)
			{
				check(ActiveCCDConstraints[Cookie.CCDConstraintIndex] == &Constraint);
				ActiveCCDConstraints[Cookie.CCDConstraintIndex] = nullptr;
				Cookie.CCDConstraintIndex = INDEX_NONE;
			}
		}

		void FCollisionConstraintAllocator::RemoveParticle(FGeometryParticleHandle* Particle)
		{
			// Removal not supported during the (parallel) collision detection phase
			check(!bInCollisionDetectionPhase);

			// Loop over all particle pairs involving this particle and tell each MidPhase 
			// that one of its particles is gone. It will get pruned at the next collision detection phase.
			// Also remove its collisions from the Active lists.
			Particle->ParticleCollisions().VisitMidPhases([this, Particle](FParticlePairMidPhase& MidPhase)
				{
					MidPhase.VisitCollisions([this](FPBDCollisionConstraint& Constraint)
						{
							RemoveActiveConstraint(Constraint);
							return ECollisionVisitorResult::Continue;
						});

					MidPhase.DetachParticle(Particle);

					return ECollisionVisitorResult::Continue;
				});
		}

		// @todo(chaos): No longer used (see ProcessNewConstraints)
		void FCollisionConstraintAllocator::SortActiveConstraints()
		{
			if (ActiveConstraints.Num())
			{
				// Sort constraints for determinism and/or improved memory-access ordering
				ActiveConstraints.Sort(
					[](const FPBDCollisionConstraintHandle& L, const FPBDCollisionConstraintHandle& R)
					{
						return L.GetContact().GetCollisionSortKey() < R.GetContact().GetCollisionSortKey();
					});

				// If we re-ordered the array, we need to update indices
				for (int32 ConstraintIndex = 0; ConstraintIndex < ActiveConstraints.Num(); ++ConstraintIndex)
				{
					FPBDCollisionConstraintContainerCookie& Cookie = ActiveConstraints[ConstraintIndex]->GetContainerCookie();

					Cookie.ConstraintIndex = ConstraintIndex;
				}
			}
		}

	}	// namespace Private
}	// namespace Chaos