// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/Collision/CollisionConstraintFlags.h"
#include "Chaos/GeometryParticles.h"
#include "Chaos/ParticleHandle.h"

// @todo(chaos): remove after IgnoreCollisionManager refactor
#include "PBDRigidsSolver.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "Chaos/PBDRigidsEvolutionGBF.h"

namespace Chaos
{

	bool FIgnoreCollisionManager::ContainsHandle(FHandleID Body0) const
	{
		return IgnoreCollisionsList.Contains(Body0);
	}

	bool FIgnoreCollisionManager::IgnoresCollision(FHandleID Body0, FHandleID Body1) const
	{
		const TArray<FIgnoreEntry>* Entries = IgnoreCollisionsList.Find(Body0);

		if(Entries)
		{
			return Entries->ContainsByPredicate([&Body1](const FIgnoreEntry& Entry)
			{
				return Entry.Id == Body1;
			});
		}

		return false;
	}

	int32 FIgnoreCollisionManager::NumIgnoredCollision(FHandleID Body0) const
	{
		const TArray<FIgnoreEntry>* Entries = IgnoreCollisionsList.Find(Body0);

		return Entries ? Entries->Num() : 0;
	}

	void FIgnoreCollisionManager::AddIgnoreCollisionsImpl(FHandleID Body0, FHandleID Body1)
	{
		TArray<FIgnoreEntry>& Entries = IgnoreCollisionsList.FindOrAdd(Body0);
		FIgnoreEntry* Entry = Entries.FindByPredicate([&Body1](const FIgnoreEntry& Entry)
		{
			return Entry.Id == Body1;
		});

		if(Entry)
		{
			Entry->Count++;
		}
		else
		{
			Entries.Add(FIgnoreEntry(Body1));
		}
	}

	int32 FIgnoreCollisionManager::RemoveIgnoreCollisionsImpl(FHandleID Body0, FHandleID Body1)
	{
		TArray<FIgnoreEntry>* Entries = IgnoreCollisionsList.Find(Body0);

		if(Entries)
		{
			int32 EntryIndex = Entries->IndexOfByPredicate([&Body1](const FIgnoreEntry& FindEntry)
			{
				return FindEntry.Id == Body1;
			});

			if(EntryIndex != INDEX_NONE)
			{
				(*Entries)[EntryIndex].Count--;

				if((*Entries)[EntryIndex].Count <= 0)
				{
					Entries->RemoveAtSwap(EntryIndex);
				}
			}

			if(Entries->Num() == 0)
			{
				IgnoreCollisionsList.Remove(Body0);
			}
			else
			{
				return Entries->Num();
			}
		}

		return 0;
	}

	bool FIgnoreCollisionManager::IgnoresCollision(const FGeometryParticleHandle* Particle0, const FGeometryParticleHandle* Particle1) const
	{
		return IgnoresCollision(Particle0->UniqueIdx(), Particle1->UniqueIdx());
	}

	void FIgnoreCollisionManager::SetIgnoreCollisionFlag(FPBDRigidParticleHandle* Rigid, const bool bUsesIgnoreCollisionManager)
	{
		if (bUsesIgnoreCollisionManager)
		{
			Rigid->SetUseIgnoreCollisionManager();
		}
		else
		{
			Rigid->ClearUseIgnoreCollisionManager();
		}
	}

	void FIgnoreCollisionManager::AddIgnoreCollisions(FGeometryParticleHandle* Particle0, FGeometryParticleHandle* Particle1)
	{
		if (Particle0 && Particle1)
		{
			FPBDRigidParticleHandle* Rigid0 = Particle0->CastToRigidParticle();
			FPBDRigidParticleHandle* Rigid1 = Particle1->CastToRigidParticle();
			if (Rigid0 || Rigid1)
			{
				const FUniqueIdx ID0 = Particle0->UniqueIdx();
				const FUniqueIdx ID1 = Particle1->UniqueIdx();

				if (Rigid0)
				{
					AddIgnoreCollisionsImpl(ID0, ID1);
					SetIgnoreCollisionFlag(Rigid0, true);
				}

				if (Rigid1)
				{
					AddIgnoreCollisionsImpl(ID1, ID0);
					SetIgnoreCollisionFlag(Rigid1, true);
				}
			}
		}
	}

	void FIgnoreCollisionManager::RemoveIgnoreCollisions(FGeometryParticleHandle* Particle0, FGeometryParticleHandle* Particle1)
	{
		if (Particle0 && Particle1)
		{
			FPBDRigidParticleHandle* Rigid0 = Particle0->CastToRigidParticle();
			FPBDRigidParticleHandle* Rigid1 = Particle1->CastToRigidParticle();
			if (Rigid0 || Rigid1)
			{
				const FUniqueIdx ID0 = Particle0->UniqueIdx();
				const FUniqueIdx ID1 = Particle1->UniqueIdx();

				if (Rigid0)
				{
					if (RemoveIgnoreCollisionsImpl(ID0, ID1) == 0)
					{
						SetIgnoreCollisionFlag(Rigid0, false);
					}
				}

				if (Rigid1)
				{
					if (RemoveIgnoreCollisionsImpl(ID1, ID0) == 0)
					{
						SetIgnoreCollisionFlag(Rigid1, false);
					}
				}
			}
		}
	}

	FGeometryParticleHandle* FIgnoreCollisionManager::GetParticleHandle(FHandleID Body, FPBDRigidsSolver& Solver)
	{
		if (FSingleParticlePhysicsProxy* Proxy = Solver.GetParticleProxy_PT(Body))
		{
			return Proxy->GetHandle_LowLevel();
		}
		return nullptr;
	}

	void FIgnoreCollisionManager::PopStorageData_Internal(int32 ExternalTimestamp)
	{
		FStorageData* StorageData;
		while(((StorageData = (StorageDataQueue.IsEmpty() ? nullptr : *StorageDataQueue.Peek())) != nullptr) && StorageData->ExternalTimestamp <= ExternalTimestamp)
		{
			for (auto& Elem : StorageData->PendingActivations)
			{
				if (PendingActivations.Contains(Elem.Key))
				{
					// the case where the key already existed should be avoided
					// but the implementation is here for completeness. 
					for (auto& Val : Elem.Value)
					{
						if (!PendingActivations[Elem.Key].Contains(Val))
						{
							PendingActivations[Elem.Key].Add(Val);
						}
					}
				}
				else
				{
					PendingActivations.Add(Elem.Key, Elem.Value);
				}
			}

			for (auto& Item : StorageData->PendingDeactivations)
			{
				PendingDeactivations.Add(Item);
			}

			StorageDataQueue.Dequeue();
			ReleaseStorageData(StorageData);
		}
	}

	void FIgnoreCollisionManager::ProcessPendingQueues(FPBDRigidsSolver& Solver)
	{

		// remove particles that have been created and destroyed
		// before the queue was ever processed. 
		TArray<FHandleID> PreculledParticles;
		if (PendingActivations.Num() && PendingDeactivations.Num())
		{
			TArray<FHandleID> DeletionList;
			
			for (const TPair<FHandleID, TArray<FHandleID>>& Elem : PendingActivations)
			{
				if (PendingDeactivations.Remove(Elem.Key))
				{
					DeletionList.Add(Elem.Key);
					PreculledParticles.Add(Elem.Key);
				}
			}

			for(FHandleID Del : DeletionList)
			{
				PendingActivations.Remove(Del);
			}
		}

		// add collision relationships for particles that have valid
		// handles, and have not already been removed from the 
		// simulation. 
		if (PendingActivations.Num())
		{
			TArray<FHandleID> DeletionList;
			for (TPair<FHandleID, TArray<FHandleID>>& Elem : PendingActivations)
			{
				for (int Index = Elem.Value.Num() - 1; Index >= 0; Index--)
				{
					if (PreculledParticles.Contains(Elem.Value[Index]))
					{
						Elem.Value.RemoveAtSwap(Index, 1);
					}
					else
					{
						FUniqueIdx ID0 = Elem.Key;
						FUniqueIdx ID1 = Elem.Value[Index];

						FGeometryParticleHandle* Particle0 = GetParticleHandle(ID0, Solver);
						FGeometryParticleHandle* Particle1 = GetParticleHandle(ID1, Solver);
						AddIgnoreCollisions(Particle0, Particle1);

						Elem.Value.RemoveAtSwap(Index, 1);
					}
				}

				if(!Elem.Value.Num())
				{
					DeletionList.Add(Elem.Key);
				}
			}

			for(FHandleID Del : DeletionList)
			{
				PendingActivations.Remove(Del);
			}
		}

		// remove relationships that exist and have been initialized. 
		if (PendingDeactivations.Num())
		{
			for (auto& Idx : PendingDeactivations)
			{
				IgnoreCollisionsList.Remove(Idx);

				if (FGeometryParticleHandle* Particle = GetParticleHandle(Idx, Solver))
				{
					if (FPBDRigidParticleHandle* Rigid = Particle->CastToRigidParticle())
					{
						SetIgnoreCollisionFlag(Rigid, false);
					}
				}
			}
			PendingDeactivations.Empty();
		}
	}

} // Chaos

