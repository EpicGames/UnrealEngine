// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/Collision/CollisionDetector.h"
#include "Chaos/PBDCollisionConstraints.h"
#include "ChaosLog.h"
#include "ChaosStats.h"

namespace Chaos
{
	const FCollisionDetectorSettings& FCollisionDetector::GetSettings() const
	{
		return CollisionContainer.GetDetectorSettings();
	}

	void FCollisionDetector::SetSettings(const FCollisionDetectorSettings& InSettings)
	{
		CollisionContainer.SetDetectorSettings(InSettings);
	}

	void FCollisionDetector::SetBoundsExpansion(const FReal InBoundsExpansion)
	{
		CollisionContainer.SetCullDistance(InBoundsExpansion);
	}
}