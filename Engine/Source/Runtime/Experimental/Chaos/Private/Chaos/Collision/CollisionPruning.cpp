// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/Collision/CollisionPruning.h"

#include "Chaos/Collision/CollisionConstraintAllocator.h"
#include "Chaos/Collision/ContactTriangles.h"
#include "Chaos/Collision/ParticleCollisions.h"
#include "Chaos/Collision/PBDCollisionConstraint.h"
#include "Chaos/HeightField.h"
#include "Chaos/ImplicitObject.h"
#include "Chaos/ImplicitObjectScaled.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/TriangleMeshImplicitObject.h"

namespace Chaos
{
	extern FRealSingle Chaos_Collision_EdgePrunePlaneDistance;

	inline FReal CalculateEdgePlaneTolarance(const FGeometryParticleHandle* Particle)
	{
		// Chaos_Collision_EdgePrunePlaneDistance is prune distance for unit sized object (100cm) so scale to the size of this object
		const FReal EdgePlaneTolerancePerCm = FReal(Chaos_Collision_EdgePrunePlaneDistance) * FReal(0.01);
		const FReal ParticleSize = Particle->HasBounds() ? Particle->LocalBounds().Extents().Min() : FReal(1);
		const FReal EdgePlaneTolerance = ParticleSize * EdgePlaneTolerancePerCm;
		return EdgePlaneTolerance;
	}

	inline FReal CalculateEdgePointBarycentricTolarance(const FGeometryParticleHandle* Particle)
	{
		return FReal(0.001);
	}

	inline bool CollisionsHasTriangleEdgeContacts(const FPBDCollisionConstraint& Collision, const int32 MeshParticleIndex)
	{
		const EContactPointType TriangleFaceContactPointType = (MeshParticleIndex == 0) ? EContactPointType::PlaneVertex : EContactPointType::VertexPlane;
		for (const FManifoldPoint& ManifoldPoint : Collision.GetManifoldPoints())
		{
			if (ManifoldPoint.ContactPoint.ContactType != TriangleFaceContactPointType)
			{
				return true;
			}
		}
		return false;
	}

	inline FVec3 CalculateTriangleNormal(const FVec3 Vertices[])
	{
		return FVec3::CrossProduct(Vertices[1] - Vertices[0], Vertices[2] - Vertices[0]).GetSafeNormal();
	}

	// Casting utility for mesh types (tri mesh and heightfield)
	// TFunctor: void(TMeshImplicitType* MeshImplicit, const FVec3& MeshScale)
	template <typename TFunctor>
	void MeshCastHelper(const FImplicitObject& Geom, const TFunctor& Func)
	{
		const EImplicitObjectType Type = Geom.GetType();
		switch (Type)
		{
		case ImplicitObjectType::TriangleMesh:
		{
			const FTriangleMeshImplicitObject* Mesh = Geom.AsAChecked<FTriangleMeshImplicitObject>();
			return Func(Mesh, FVec3(1));
		}
		case ImplicitObjectType::HeightField:
		{
			const FHeightField* Mesh = Geom.AsAChecked<FHeightField>();
			return Func(Mesh, FVec3(1));
		}
		case ImplicitObjectType::IsScaled | ImplicitObjectType::TriangleMesh:
		{
			const TImplicitObjectScaled<FTriangleMeshImplicitObject>* ScaledMesh = Geom.AsA<TImplicitObjectScaled<FTriangleMeshImplicitObject>>();
			return Func(ScaledMesh->GetUnscaledObject(), ScaledMesh->GetScale());
		}
		case ImplicitObjectType::IsScaled | ImplicitObjectType::HeightField:
		{
			const TImplicitObjectScaled<FHeightField>* ScaledMesh = Geom.AsA<TImplicitObjectScaled<FHeightField>>();
			return Func(ScaledMesh->GetUnscaledObject(), ScaledMesh->GetScale());
		}
		case ImplicitObjectType::IsInstanced | ImplicitObjectType::TriangleMesh:
		{
			const TImplicitObjectInstanced<FTriangleMeshImplicitObject>* ScaledMesh = Geom.AsA<TImplicitObjectInstanced<FTriangleMeshImplicitObject>>();
			return Func(ScaledMesh->GetInstancedObject(), FVec3(1));
		}
		case ImplicitObjectType::IsInstanced | ImplicitObjectType::HeightField:
		{
			const TImplicitObjectInstanced<FHeightField>* ScaledMesh = Geom.AsA<TImplicitObjectInstanced<FHeightField>>();
			return Func(ScaledMesh->GetInstancedObject(), FVec3(1));
		}
		}

		// Should only be called with a known mesh
		check(false);
	}

	// Extract the triangle with the specified index (from a collision result) from a mesh or heightfield
	void GetTransformedMeshTriangle(
		const FImplicitObject* InImplicit, const int32 TriangleIndex, const FRigidTransform3& InTransform,
		FVec3 OutVertices[],
		int32 OutVertexIndices[])
	{
		MeshCastHelper(*InImplicit,
			[&InTransform, TriangleIndex, &OutVertices, OutVertexIndices](const auto& MeshImplicit, const FVec3& MeshScale) -> void
			{
				const FTransform Transform = FTransform(InTransform.GetRotation(), InTransform.GetLocation(), MeshScale);
				FTriangle Triangle;
				MeshImplicit->GetTransformedTriangle(TriangleIndex, Transform, Triangle, OutVertexIndices[0], OutVertexIndices[1], OutVertexIndices[2]);
				OutVertices[0] = Triangle.GetVertex(0);
				OutVertices[1] = Triangle.GetVertex(1);
				OutVertices[2] = Triangle.GetVertex(2);
			});
	}

	// Loop over edge collisions, then all plane collisions and remove the edge collision if it is hidden by a plane collision
	// NOTE: We only look at plane collisions where the other shape owns the plane.
	// @todo(chaos): this should probably only disable individual manifold points
	// @todo(chaos): we should probably only reject edges if the plane contact is also close to the edge contact
	// @todo(chaos): we should also try to eliminate face contacts from sub-surface faces
	// @todo(chaos): perf issue: this processes contacts in world space, but we don't calculated that data until Gather. Fix this.
	void FParticleEdgeCollisionPruner::Prune()
	{
		const FReal EdgePlaneTolerance = CalculateEdgePlaneTolarance(Particle);

		FParticleCollisions& ParticleCollisions = Particle->ParticleCollisions();

		ParticleCollisions.VisitCollisions(
			[this, &ParticleCollisions, EdgePlaneTolerance](FPBDCollisionConstraint& EdgeCollision)
			{
				if (EdgeCollision.IsEnabled())
				{
					const int32 EdgeOtherShapeIndex = (EdgeCollision.GetParticle0() == Particle) ? 1 : 0;
					const EContactPointType VertexContactType = (EdgeOtherShapeIndex == 0) ? EContactPointType::VertexPlane : EContactPointType::PlaneVertex;

					// We skip meshes - they are handled by FParticleMeshCollisionPruner
					if (EdgeCollision.GetImplicit(EdgeOtherShapeIndex)->IsUnderlyingMesh())
					{
						return ECollisionVisitorResult::Continue;
					}

					for (const FManifoldPoint& EdgeManifoldPoint : EdgeCollision.GetManifoldPoints())
					{
						const bool bIsEdgeContact = (EdgeManifoldPoint.ContactPoint.ContactType == EContactPointType::EdgeEdge);

						if (bIsEdgeContact)
						{
							const FRigidTransform3& EdgeTransform = (EdgeOtherShapeIndex == 0) ? EdgeCollision.GetShapeWorldTransform0() : EdgeCollision.GetShapeWorldTransform1();
							const FVec3 EdgePos = EdgeTransform.TransformPositionNoScale(FVec3(EdgeManifoldPoint.ContactPoint.ShapeContactPoints[EdgeOtherShapeIndex]));

							// Loop over plane collisions
							ECollisionVisitorResult PlaneResult = ParticleCollisions.VisitConstCollisions(
								[this, &EdgeCollision, &EdgePos, EdgePlaneTolerance](const FPBDCollisionConstraint& PlaneCollision)
								{
									if ((&PlaneCollision != &EdgeCollision) && PlaneCollision.IsEnabled())
									{
										const int32 PlaneOtherShapeIndex = (PlaneCollision.GetParticle0() == Particle) ? 1 : 0;
										const EContactPointType PlaneContactType = (PlaneOtherShapeIndex == 0) ? EContactPointType::PlaneVertex : EContactPointType::VertexPlane;
										const FRigidTransform3& PlaneTransform = (PlaneOtherShapeIndex == 0) ? PlaneCollision.GetShapeWorldTransform0() : PlaneCollision.GetShapeWorldTransform1();

										for (const FManifoldPoint& PlaneManifoldPoint : PlaneCollision.GetManifoldPoints())
										{
											if (PlaneManifoldPoint.ContactPoint.ContactType == PlaneContactType)
											{
												// If the edge position is in the plane, disable it
												const FVec3 PlanePos = PlaneTransform.TransformPositionNoScale(FVec3(PlaneManifoldPoint.ContactPoint.ShapeContactPoints[PlaneOtherShapeIndex]));
												const FVec3 PlaneNormal = PlaneCollision.GetShapeWorldTransform1().TransformVectorNoScale(FVec3(PlaneManifoldPoint.ContactPoint.ShapeContactNormal));

												const FVec3 EdgePlaneDelta = EdgePos - PlanePos;
												const FReal EdgePlaneDistance = FVec3::DotProduct(EdgePlaneDelta, PlaneNormal);
												if (FMath::Abs(EdgePlaneDistance) < EdgePlaneTolerance)
												{
													// The edge contact is hidden by a plane contact so disable it and stop the inner loop
													EdgeCollision.SetDisabled(true);
													return ECollisionVisitorResult::Stop;
												}
											}
										}
									}
									return ECollisionVisitorResult::Continue;
								});

							// If we disabled this constraint, move to the next one and ignore remaining manifold points
							if (PlaneResult == ECollisionVisitorResult::Stop)
							{
								break;
							}
						}
					}
				}
				return ECollisionVisitorResult::Continue;
			});
	}

	void FParticleMeshCollisionPruner::Prune()
	{
		const FReal EdgePointBarycentricTolerance = CalculateEdgePointBarycentricTolarance(Particle);

		// NOTE: We are assuming that our particle does not have mesh collisions and we are just checking for collisions
		// with other meshes. This is ok since we don't support meshes on dynamic particles.
		FParticleCollisions& ParticleCollisions = Particle->ParticleCollisions();

		// Visit all pairs of collisions where we are colliding with a mesh.
		// Eliminate edge collisions against one of the meshes if the edge is shared with the other mesh
		// and the edge is in the plane of a face that we are colliding with on that mesh.
		// Fix normals if we discover than an edge collision is on an edge that is shared with a triangle on another mesh
		ParticleCollisions.VisitCollisions(
			[this, &ParticleCollisions, EdgePointBarycentricTolerance](FPBDCollisionConstraint& MeshCollisionA)
			{
				// Skip non-mesh collisions
				const int32 MeshParticleIndexA = 1;	// Meshes are always the second shape
				const bool bIsMeshA = MeshCollisionA.GetImplicit(MeshParticleIndexA)->IsUnderlyingMesh();
				if (!bIsMeshA)
				{
					return ECollisionVisitorResult::Continue;
				}

				// For the outer loop, we only care about collisions with triangle edges (i.e., not the triangle face).
				if (!CollisionsHasTriangleEdgeContacts(MeshCollisionA, MeshParticleIndexA))
				{
					return ECollisionVisitorResult::Continue;
				}

				// Collision A is with a mesh and has edges. Now see if any other mesh collision invalidate the contact or require the normal to be fixed
				ParticleCollisions.VisitCollisions(
					[this, &MeshCollisionA, MeshParticleIndexA, EdgePointBarycentricTolerance](FPBDCollisionConstraint& MeshCollisionB)
					{
						const int32 MeshParticleIndexB = 1; // Meshes are always the second shape

						// Skip collisions pairs on the same mesh (this implicitly skips the A-A case)
						if (MeshCollisionA.GetImplicit(MeshParticleIndexA) == MeshCollisionB.GetImplicit(MeshParticleIndexB))
						{
							return ECollisionVisitorResult::Continue;
						}

						// Skip non-mesh collisions
						const bool bIsMeshB = MeshCollisionB.GetImplicit(MeshParticleIndexB)->IsUnderlyingMesh();
						if (!bIsMeshB)
						{
							return ECollisionVisitorResult::Continue;
						}

						const EContactPointType TriangleFaceContactPointTypeB = (MeshParticleIndexB == 0) ? EContactPointType::PlaneVertex : EContactPointType::VertexPlane;

						// Visit the edge contact points on A...
						for (int32 ManifoldPointIndexA = 0; ManifoldPointIndexA < MeshCollisionA.NumManifoldPoints(); ++ManifoldPointIndexA)
						{
							FManifoldPoint& ManifoldPointA = MeshCollisionA.GetManifoldPoint(ManifoldPointIndexA);
							const EContactPointType TriangleFaceContactPointTypeA = (MeshParticleIndexA == 0) ? EContactPointType::PlaneVertex : EContactPointType::VertexPlane;
							if (ManifoldPointA.ContactPoint.ContactType == TriangleFaceContactPointTypeA)
							{
								// Not an edge
								continue;
							}

							// Visit all the face contacts on B...
							for (const FManifoldPoint& ManifoldPointB : MeshCollisionB.GetManifoldPoints())
							{
								// The transforms of the two meshes
								const FRigidTransform3& MeshTransformA = MeshCollisionA.GetShapeWorldTransform(MeshParticleIndexA);
								const FRigidTransform3& MeshTransformB = MeshCollisionB.GetShapeWorldTransform(MeshParticleIndexB);

								// Get the world-space triangle for collision B
								FVec3 VerticesB[3];
								int32 VertexIndicesB[3];
								GetTransformedMeshTriangle(MeshCollisionB.GetImplicit(MeshParticleIndexB), ManifoldPointB.ContactPoint.FaceIndex, MeshTransformB, VerticesB, VertexIndicesB);

								// World-space position of the edge contact A
								const FVec3 EdgePosA = MeshTransformA.TransformPositionNoScale(FVec3(ManifoldPointA.ContactPoint.ShapeContactPoints[MeshParticleIndexA]));

								// Is the edge contact on A on one of the edges of B?
								int32 EdgeVertexIndicesB[2];
								const bool bIsOnEdgeB = GetTriangleEdgeVerticesAtPosition(EdgePosA, VerticesB, EdgeVertexIndicesB[0], EdgeVertexIndicesB[1], EdgePointBarycentricTolerance);
								
								// If ManifoldPointA is not on and edge/vertex of triangleB then we won't be disabling A here
								if (!bIsOnEdgeB)
								{
									continue;
								}

								// Get the world-space triangle for collision A
								FVec3 VerticesA[3];
								int32 VertexIndicesA[3];
								GetTransformedMeshTriangle(MeshCollisionA.GetImplicit(MeshParticleIndexA), ManifoldPointA.ContactPoint.FaceIndex, MeshTransformA, VerticesA, VertexIndicesA);

								// If ManifoldPoint A is on Triangle B's edge, and we collided with B's face, fix the normal on A
								const FVec3 FaceNormalA = CalculateTriangleNormal(VerticesA);
								if (bIsOnEdgeB && (ManifoldPointB.ContactPoint.ContactType == TriangleFaceContactPointTypeB))
								{
									ManifoldPointA.ContactPoint.ShapeContactNormal = MeshTransformA.InverseTransformVectorNoScale(FaceNormalA);
									break;
								}

								// @todo(chaos): We are only handling edge contacts here, not vertex contacts which are much harder to deal with
								// because we need to know all triangles that share the vertex in order to correct the normal
								if ((EdgeVertexIndicesB[0] != INDEX_NONE) && (EdgeVertexIndicesB[1] != INDEX_NONE))
								{
									// It is common for the normals to be almost the same (near flat surfaces) and if that's the case just clamp the contact normal to the face normal
									const FVec3 FaceNormalB = CalculateTriangleNormal(VerticesB);
									const FReal FaceNormalADotB = FVec3::DotProduct(FaceNormalA, FaceNormalB);
									const FReal FlatNormalThreshold = FReal(0.999);	// ~2deg
									if (FaceNormalADotB > FlatNormalThreshold)
									{
										ManifoldPointA.ContactPoint.ShapeContactNormal = MeshTransformA.InverseTransformVectorNoScale(FaceNormalA);
										break;
									}

									// Calculate the edge plane on triangle B
									const FVec3 EdgeVectorB = VerticesB[EdgeVertexIndicesB[1]] - VerticesB[EdgeVertexIndicesB[0]];
									const FVec3 EdgePlaneNormalB = FVec3::CrossProduct(EdgeVectorB, FaceNormalB);	// NOTE: Not normalized!!
									const FReal EdgePlaneNormalLenSqB = EdgePlaneNormalB.SizeSquared();

									// If the Face Normal on Triangle A points into the Edge Plane Normal from Triangle B, we are at a concave edge
									// in which case we set the normal to the face normal
									const FVec3 ContactNormalA = MeshTransformA.TransformVectorNoScale(FVec3(ManifoldPointA.ContactPoint.ShapeContactNormal));
									const FReal FaceNormalADotEdgePlaneNormalB = FVec3::DotProduct(FaceNormalA, EdgePlaneNormalB);	// Scaled by EdgePlaneNormal.Size()
									const FReal ConcaveNormalThresholdSq = -FMath::Square(0.01f);	// 0.5 deg
									if (FaceNormalADotEdgePlaneNormalB * FMath::Abs(FaceNormalADotEdgePlaneNormalB) < ConcaveNormalThresholdSq * EdgePlaneNormalLenSqB)			// Accounts for non-normalized NormalADotEdgePlaneNormalB
									{
										ManifoldPointA.ContactPoint.ShapeContactNormal = MeshTransformA.InverseTransformVectorNoScale(FaceNormalA);
										break;
									}

									// If the faces are not concave, force the normal to the face normal if it points into the edge plane
									// NOTE: We assume that the contact normal does not point into the edge plane on A because collision detection should ensure that
									const FReal ContactNormalADotEdgePlaneNormalB = FVec3::DotProduct(ContactNormalA, EdgePlaneNormalB);
									if (ContactNormalADotEdgePlaneNormalB < FReal(0))
									{
										ManifoldPointA.ContactPoint.ShapeContactNormal = MeshTransformA.InverseTransformVectorNoScale(FaceNormalA);
										break;
									}
								}
							}
						}

						// If we disable all of A's edges, we don't need to go on
						if (!CollisionsHasTriangleEdgeContacts(MeshCollisionA, MeshParticleIndexA))
						{
							return ECollisionVisitorResult::Stop;
						}
						return ECollisionVisitorResult::Continue;
					});

				return ECollisionVisitorResult::Continue;
			});
	}


	/**
	 * @brief Information about a contact point held in a ContactGroup (points in the same plane)
	*/
	class FContactGroupManifoldPoint
	{
	public:
		FPBDCollisionConstraint* Collision;
		int32 ManifoldPointIndex;
		FVec3 ContactNormal;
		FVec3 ContactPosition;
	};

	/**
	 * @brief A set of contacts in the same plane
	*/
	class FContactGroup
	{
	public:
		FContactGroup()
			: PlanePosition()
			, PlaneNormal()
			, UpDotNormal(0)
			, bActive(false)
		{
		}

		void Add(FPBDCollisionConstraint& Collision, const int32 ManifoldPointIndex, const FVec3& InContactPosition, const FVec3& InContactNormal, const FReal InUpDotNormal)
		{
			if (ManifoldPoints.Num() == 0)
			{
				PlanePosition = InContactPosition;
				PlaneNormal = InContactNormal;
				UpDotNormal = InUpDotNormal;
				bActive = true;
			}

			FContactGroupManifoldPoint& Point = ManifoldPoints[ManifoldPoints.AddDefaulted()];
			Point.Collision = &Collision;
			Point.ManifoldPointIndex = ManifoldPointIndex;
			Point.ContactPosition = InContactPosition;
			Point.ContactNormal = InContactNormal;
		}

		TArray<FContactGroupManifoldPoint> ManifoldPoints;
		FVec3 PlanePosition;
		FVec3 PlaneNormal;
		FReal UpDotNormal;
		bool bActive;
	};

	/**
	 * @brief All the contacts on a body, held in sets where all contacts in each set are in the same plane
	*/
	class FContactGroupContainer
	{
	public:
		FContactGroupContainer(const FVec3& InUpVector)
			: UpVector(InUpVector)
		{
		}
		
		void Add(FPBDCollisionConstraint& Collision, const int32 ManifoldPointIndex, const FVec3& ContactPos, const FVec3& ContactNormal)
		{
			FContactGroup* ContactGroup = FindContactGroup(ContactPos, ContactNormal);
			if (ContactGroup == nullptr)
			{
				ContactGroup = &ContactGroups[ContactGroups.AddDefaulted()];
			}

			const FReal UpDotNormal = FVec3::DotProduct(UpVector, ContactNormal);
			ContactGroup->Add(Collision, ManifoldPointIndex, ContactPos, ContactNormal, UpDotNormal);
		}

		FContactGroup* FindContactGroup(const FVec3& ContactPos, const FVec3& ContactNormal)
		{
			const FReal NormalProjectionThreshold = FReal(0.99);	// 8deg
			const FReal PlaneDistanceThreshold = FReal(5);

			for (FContactGroup& ContactGroup : ContactGroups)
			{
				const FReal NormalProjection = FVec3::DotProduct(ContactNormal, ContactGroup.PlaneNormal);
				if (NormalProjection > NormalProjectionThreshold)
				{
					const FReal PlaneDistance = FVec3::DotProduct(ContactPos - ContactGroup.PlanePosition, ContactGroup.PlaneNormal);
					if (FMath::Abs(PlaneDistance) < PlaneDistanceThreshold)
					{
						return &ContactGroup;
					}
				}
			}
			return nullptr;
		}

		void SortContactGroups()
		{
			ContactGroups.Sort([](const FContactGroup& L, const FContactGroup& R)
			{
				return L.UpDotNormal > R.UpDotNormal;
			});
		}

		TArray<FContactGroup, TInlineAllocator<4>> ContactGroups;
		FVec3 UpVector;
	};


	void FParticleSubSurfaceCollisionPruner::Prune(const FVec3& UpVector)
	{
		// Chaos_Collision_EdgePrunePlaneDistance is prune distance for unit sized object (100cm) so scale by size of this object
		const FReal PlaneDistanceTolerancePerCm = FReal(Chaos_Collision_EdgePrunePlaneDistance) * FReal(0.01);
		const FReal PlaneDistanceSize = Particle->HasBounds() ? Particle->LocalBounds().Extents().Min() : FReal(1);
		const FReal PlaneDistanceTolerance = PlaneDistanceSize * PlaneDistanceTolerancePerCm;
		const FReal PlaneNormalTolerance = FReal(0.95);

		FContactGroupContainer ContactGroupContainer(UpVector);

		FParticleCollisions& ParticleCollisions = Particle->ParticleCollisions();

		ParticleCollisions.VisitCollisions(
			[this, &ContactGroupContainer](FPBDCollisionConstraint& Collision)
			{
				if (Collision.IsEnabled())
				{
					const int32 OtherShapeIndex = (Collision.GetParticle0() == Particle) ? 1 : 0;
					const FRigidTransform3& ContactTransform = (OtherShapeIndex == 0) ? Collision.GetShapeWorldTransform0() : Collision.GetShapeWorldTransform1();
					FConstGenericParticleHandle P0 = Collision.GetParticle0();
					FConstGenericParticleHandle P1 = Collision.GetParticle1();
					FConstGenericParticleHandle OtherParticle = (OtherShapeIndex == 0) ? Collision.GetParticle0() : Collision.GetParticle1();

					// Only reject sub-surface contacts against static or kinematic objects
					if (!OtherParticle->IsDynamic())
					{
						for (int32 ManifoldPointIndex = 0; ManifoldPointIndex < Collision.NumManifoldPoints(); ++ManifoldPointIndex)
						{
							const FManifoldPoint& ManifoldPoint = Collision.GetManifoldPoint(ManifoldPointIndex);

							const FVec3 ContactPos = ContactTransform.TransformPositionNoScale(FVec3(ManifoldPoint.ContactPoint.ShapeContactPoints[OtherShapeIndex]));
							const FVec3 ContactNormal = Collision.GetShapeWorldTransform1().TransformVectorNoScale(FVec3(ManifoldPoint.ContactPoint.ShapeContactNormal));

							ContactGroupContainer.Add(Collision, ManifoldPointIndex, ContactPos, ContactNormal);
						}
					}
				}
				return ECollisionVisitorResult::Continue;
			});

		// Sort the groups by the most vertical normal so that we preferentially keep horizontal planes (wrt the UpVector)
		ContactGroupContainer.SortContactGroups();

		// If all contacts in one group lie beneath the plane of another group, reject it
		// Loop over all groups (i.e., contact planes)
		for (int32 ContactGroupIndex = 0; ContactGroupIndex < ContactGroupContainer.ContactGroups.Num(); ++ContactGroupIndex)
		{
			const FContactGroup& ContactGroup = ContactGroupContainer.ContactGroups[ContactGroupIndex];

			// @todo(chaos): assuming here that the contacts are not all in a line...do we need to check for that?
			if (!ContactGroup.bActive)
			{
				continue;
			}
			
			// Loop over all other contact groups
			for (int32 RejectCandidateContactGroupIndex = ContactGroupIndex + 1; RejectCandidateContactGroupIndex < ContactGroupContainer.ContactGroups.Num(); ++RejectCandidateContactGroupIndex)
			{
				FContactGroup& RejectCandidateContactGroup = ContactGroupContainer.ContactGroups[RejectCandidateContactGroupIndex];
				if (!RejectCandidateContactGroup.bActive)
				{
					continue;
				}
				
				// Are any point in the candidate group above the plane?
				bool bPointsAbovePlane = false;
				for (FContactGroupManifoldPoint& RejectCandidateManifoldPoint : RejectCandidateContactGroup.ManifoldPoints)
				{
					const FReal PlaneDistance = FVec3::DotProduct(RejectCandidateManifoldPoint.ContactPosition - ContactGroup.PlanePosition, ContactGroup.PlaneNormal);
					if (PlaneDistance > PlaneDistanceTolerance)
					{
						bPointsAbovePlane = true;
						break;
					}
				}

				// If all point in the candidate group are below the plane, reject them
				if (!bPointsAbovePlane)
				{
					for (FContactGroupManifoldPoint& RejectCandidateManifoldPoint : RejectCandidateContactGroup.ManifoldPoints)
					{
						RejectCandidateManifoldPoint.Collision->DisableManifoldPoint(RejectCandidateManifoldPoint.ManifoldPointIndex);
					}
					RejectCandidateContactGroup.bActive = false;
				}
			}
		}
	}
}