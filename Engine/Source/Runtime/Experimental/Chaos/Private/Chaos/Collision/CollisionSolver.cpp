// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/Collision/CollisionSolver.h"

/**
 * @brief This file used to contain collision solver code for the GbfPbd solver (which was no longer functional)
 * @see PBDCollisionSolver.cpp for the new solver
*/
