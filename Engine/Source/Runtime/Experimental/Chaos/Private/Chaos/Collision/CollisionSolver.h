// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

/**
 * @brief This file used to contain collision solver code for the GbfPbd solver (which was no longer functional)
 * @see PBDCollisionSolver.h for the new solver
*/

