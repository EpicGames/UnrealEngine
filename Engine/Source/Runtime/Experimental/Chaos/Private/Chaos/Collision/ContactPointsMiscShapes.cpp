// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/Collision/ContactPointsMiscShapes.h"
#include "Chaos/CastingUtilities.h"
#include "Chaos/Collision/ContactPoint.h"
#include "Chaos/Collision/GJKContactPoint.h"
#include "Chaos/Convex.h"
#include "Chaos/Defines.h"
#include "Chaos/GJK.h"
#include "Chaos/GJKShape.h"
#include "Chaos/HeightField.h"
#include "Chaos/ImplicitObjectScaled.h"
#include "Chaos/Sphere.h"

DECLARE_CYCLE_STAT(TEXT("Collisions::CapsuleHeightFieldContactPoint"), STAT_Collisions_CapsuleHeightFieldContactPoint, STATGROUP_ChaosCollision);
DECLARE_CYCLE_STAT(TEXT("Collisions::CapsuleTriangleMeshContactPoint"), STAT_Collisions_CapsuleTriangleMeshContactPoint, STATGROUP_ChaosCollision);
DECLARE_CYCLE_STAT(TEXT("Collisions::CapsuleTriangleMeshSweptContactPoint"), STAT_Collisions_CapsuleTriangleMeshSweptContactPoint, STATGROUP_ChaosCollision);
DECLARE_CYCLE_STAT(TEXT("Collisions::ConvexHeightFieldContactPoint"), STAT_Collisions_ConvexHeightFieldContactPoint, STATGROUP_ChaosCollision);
DECLARE_CYCLE_STAT(TEXT("Collisions::ConvexTriangleMeshContactPoint"), STAT_Collisions_ConvexTriangleMeshContactPoint, STATGROUP_ChaosCollision);
DECLARE_CYCLE_STAT(TEXT("Collisions::ConvexTriangleMeshSweptContactPoint"), STAT_Collisions_ConvexTriangleMeshSweptContactPoint, STATGROUP_ChaosCollision);


extern int32 ConstraintsDetailedStats;

namespace Chaos
{
	template <typename GeometryB>
	FContactPoint GJKImplicitSweptContactPoint(const FImplicitObject& A, const FRigidTransform3& AStartTransform, const GeometryB& B, const FRigidTransform3& BTransform, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& OutTOI)
	{
		FContactPoint Contact;
		const FRigidTransform3 AToBTM = AStartTransform.GetRelativeTransform(BTransform);
		const FVec3 LocalDir = BTransform.InverseTransformVectorNoScale(Dir);

		FReal TOI = TNumericLimits<FReal>::Max();
		FReal Phi = TNumericLimits<FReal>::Max();
		int32 FaceIndex = INDEX_NONE;
		FVec3 FaceNormal;
		FVec3 Location, Normal;

		Utilities::CastHelper(A, AStartTransform, [&](const auto& ADowncast, const FRigidTransform3& AFullTM)
			{
				if (B.SweepGeomCCD(ADowncast, AToBTM, LocalDir, Length, IgnorePenetration, TargetPenetration, TOI, Phi, Location, Normal, FaceIndex, FaceNormal))
				{
					Contact.ShapeContactPoints[0] = AToBTM.InverseTransformPosition(Location);
					Contact.ShapeContactPoints[1] = Location;
					Contact.ShapeContactNormal = Normal;
					Contact.Phi = Phi;
					Contact.FaceIndex = FaceIndex;
					OutTOI = TOI;
				}
			});

		return Contact;
	}

	// GJKRaycast Time output depends on whether there was initial overlap.
	// For no initial overlap, Time is the Distance along the sweep where the contact occurs
	// For initial overlap, Time is the Phi value (separation, so negative)
	void ComputeSweptContactPhiAndTOIHelper(const FReal DirDotNormal, const FReal& Length, const FReal& HitTime, FReal& OutTOI, FReal& OutPhi)
	{
		if (HitTime >= 0.0f)
		{
			// We subtract length to get the total penetration at at end of frame.
			// Project penetration vector onto geometry normal for correct phi.
			FReal Dot = -DirDotNormal;
			OutPhi = (HitTime - Length) * Dot;

			// TOI is between [0,1], used to compute particle position
			OutTOI = HitTime / Length;
		}
		else
		{
			// Initial overlap case:
			// TOI = 0 as we are overlapping at X.
			// OutTime is penetration value of MTD.
			OutPhi = HitTime;
			OutTOI = 0.0f;
		}
	}

	void ComputeSweptContactPhiAndTOIHelper(const FVec3& ContactNormal, const FVec3& Dir, const FReal& Length, const FReal& HitTime, FReal& OutTOI, FReal& OutPhi)
	{
		const FReal DirDotNormal = FMath::Abs(FVec3::DotProduct(ContactNormal, Dir));
		ComputeSweptContactPhiAndTOIHelper(DirDotNormal, Length, HitTime, OutTOI, OutPhi);
	}

	// Calculate Phi at the start and end of a raycast, given the ray information and sweep result
	bool ComputeSweptContactStartAndEndPhi(const FReal TOI, const FReal Phi, const FReal DirDotNormal, const FReal Length, FReal& StartPhi, FReal& EndPhi)
	{
		StartPhi = TNumericLimits<FReal>::Max();
		EndPhi = TNumericLimits<FReal>::Max();
		if (TOI <= FReal(1))
		{
			// Modify TOI so we ignore separating and shallow CCD contacts
			// NOTE (See GJKRaycast2): If initially penetrating, Phi is the separation at the start of the sweep.
			// But if not initially penetrating Phi is the separation at the end of the sweep
			// @todo(chaos): The way Phi is set for sweeps is quite confusing - can we make this better?
			if (TOI <= FReal(0))
			{
				StartPhi = Phi;
				EndPhi = StartPhi + DirDotNormal * Length;
			}
			else
			{
				EndPhi = Phi;
				StartPhi = EndPhi - DirDotNormal * Length;
			}
			return true;
		}
		return false;
	}

	// Use the ray start and end Phi to calculate the time when Phi was equal to TargetPhi. Also ignore separating rays and
	// those where penatration is belocw some threshold at the end of the sweep.
	FReal ComputeSweptContactTimeToTargetPhi(const FReal StartPhi, const FReal EndPhi, const FReal IgnorePhi, const FReal TargetPhi)
	{
		const FReal InfiniteTOI = TNumericLimits<FReal>::Max();
		const FReal MovementTolerance = KINDA_SMALL_NUMBER;

		// If we end up separated at TOI=1 ignore the contact
		if (EndPhi > 0)
		{
			return InfiniteTOI;
		}

		// If we penetrate by less than the IgnorePhi, treat it as TOI=1. This mean no CCD impulse and the non-CCD 
		// solve is expected to handle it. This improves the behaviour when we are sliding along a surface at
		// above CCD speeds - we don't want to handle TOI events with the floor
		if (EndPhi > IgnorePhi)
		{
			return FReal(1);
		}

		// If we penetrate by more than the TargetPhi we roll back to the TOI-plus-a-bit leaving some penetration
		// for the main solver to resolve. Note that is CCDRemainderPenetration is non-zero, secondary sweeps are
		// more likely to encounter an bad edge collision.
		const FReal TOI = (TargetPhi - StartPhi) / (EndPhi - StartPhi);

		return FMath::Clamp(TOI, FReal(0), FReal(1));
	}

	// Modify the time of impact so that the contact depth is TargetPenetration. If penetration at T=1 is less than TargetPenetration, TOI will be "infinity".
	// Returns true if we have a TOI less than or equal to 1 (i.e., a contact at TargetPenetration or more that needs CCD processing)
	bool ComputeSweptContactTOIAndPhiAtTargetPenetration(const FReal DirDotNormal, const FReal SweepLength, const FReal HitDistance, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& OutTOI, FReal& OutPhi)
	{
		// Parse the GJK result to get TOI and Phi (Interpretation depends on whether TOI=0 or not - see GJKRaycast2)
		FReal HitTOI, HitPhi;
		ComputeSweptContactPhiAndTOIHelper(DirDotNormal, SweepLength, HitDistance, HitTOI, HitPhi);

		// Calculate the Phis at the start and end of the sweep
		FReal StartPhi, EndPhi;
		ComputeSweptContactStartAndEndPhi(HitTOI, HitPhi, DirDotNormal, SweepLength, StartPhi, EndPhi);

		// Calculate the time to reach a depth of TargetPenetration (ignoring those penetrating by less than IgnorePenetration at the end)
		const FReal IgnorePhi = -IgnorePenetration;
		const FReal TargetPhi = -TargetPenetration;
		const FReal TargetTOI = ComputeSweptContactTimeToTargetPhi(StartPhi, EndPhi, IgnorePhi, TargetPhi);

		OutTOI = TargetTOI;
		OutPhi = (TargetTOI == 0) ? StartPhi : TargetPhi;

		return (OutTOI <= FReal(1));
	}

	bool ComputeSweptContactTOIAndPhiAtTargetPenetration(const FVec3& ContactNormal, const FVec3& Dir, const FReal SweepLength, const FReal HitDistance, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& OutTOI, FReal& OutPhi)
	{
		const FReal DirDotNormal = FVec3::DotProduct(ContactNormal, Dir);
		return ComputeSweptContactTOIAndPhiAtTargetPenetration(DirDotNormal, SweepLength, HitDistance, IgnorePenetration, TargetPenetration, OutTOI, OutPhi);
	}

	// Calculate TOI to leave objects penetrating by TargetPhi. This is used by the legacy path that finds the first contacting shape pair and then ignores all others after that
	// even if they would have a smaller TOI at TargetPenetration. It is only called if the low level sweep code has not already called ComputeSweptContactTOIAndPhiAtTargetPenetration. 
	// See CVars::bCCDNewTargetDepthMode
	void LegacyComputeSweptContactTOIAndPhiAtTargetPenetration(const FReal DirDotNormal, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& InOutTOI, FReal& InOutPhi)
	{
		if (InOutTOI <= 1)
		{
			if (InOutTOI != 0)
			{
				InOutPhi = (FReal(1) - InOutTOI) * Length * DirDotNormal;
			}
			FReal StartPhi, EndPhi;
			ComputeSweptContactStartAndEndPhi(InOutTOI, InOutPhi, DirDotNormal, Length, StartPhi, EndPhi);
			InOutTOI = ComputeSweptContactTimeToTargetPhi(StartPhi, EndPhi, -IgnorePenetration, -TargetPenetration);
			InOutPhi = (InOutTOI == 0) ? StartPhi : -TargetPenetration;
		}
	}

	template <typename GeometryA, typename GeometryB>
	FContactPoint GJKImplicitContactPoint(const FImplicitObject& A, const FRigidTransform3& ATransform, const GeometryB& B, const FRigidTransform3& BTransform, const FReal CullDistance)
	{
		FContactPoint Contact;
		const FRigidTransform3 AToBTM = ATransform.GetRelativeTransform(BTransform);

		FReal ContactPhi = FLT_MAX;
		FVec3 Location, Normal;
		int32 ContactFaceIndex = INDEX_NONE;
		if (const TImplicitObjectScaled<GeometryA>* ScaledConvexImplicit = A.template GetObject<const TImplicitObjectScaled<GeometryA> >())
		{
			if (B.GJKContactPoint(*ScaledConvexImplicit, AToBTM, CullDistance, Location, Normal, ContactPhi, ContactFaceIndex))
			{
				Contact.ShapeContactPoints[0] = AToBTM.InverseTransformPosition(Location);
				Contact.ShapeContactPoints[1] = Location - ContactPhi * Normal;
				Contact.ShapeContactNormal = Normal;
				Contact.Phi = ContactPhi;
				Contact.FaceIndex = ContactFaceIndex;
			}
		}
		else if (const TImplicitObjectInstanced<GeometryA>* InstancedConvexImplicit = A.template GetObject<const TImplicitObjectInstanced<GeometryA> >())
		{
			if (const GeometryA* InstancedInnerObject = static_cast<const GeometryA*>(InstancedConvexImplicit->GetInstancedObject()))
			{
				if (B.GJKContactPoint(*InstancedInnerObject, AToBTM, CullDistance, Location, Normal, ContactPhi, ContactFaceIndex))
				{
					Contact.ShapeContactPoints[0] = AToBTM.InverseTransformPosition(Location);
					Contact.ShapeContactPoints[1] = Location - ContactPhi * Normal;
					Contact.ShapeContactNormal = Normal;
					Contact.Phi = ContactPhi;
					Contact.FaceIndex = ContactFaceIndex;
				}
			}
		}
		else if (const GeometryA* ConvexImplicit = A.template GetObject<const GeometryA>())
		{
			if (B.GJKContactPoint(*ConvexImplicit, AToBTM, CullDistance, Location, Normal, ContactPhi, ContactFaceIndex))
			{
				Contact.ShapeContactPoints[0] = AToBTM.InverseTransformPosition(Location);
				Contact.ShapeContactPoints[1] = Location - ContactPhi * Normal;
				Contact.ShapeContactNormal = Normal;
				Contact.Phi = ContactPhi;
				Contact.FaceIndex = ContactFaceIndex;
			}
		}

		return Contact;
	}

	// A is the implicit here, we want to return a contact point on B (trimesh)
	template <typename GeometryA>
	FContactPoint GJKImplicitScaledTriMeshSweptContactPoint(const FImplicitObject& A, const FRigidTransform3& AStartTransform, const TImplicitObjectScaled<FTriangleMeshImplicitObject>& B, const FRigidTransform3& BTransform, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& OutTOI)
	{
		FContactPoint Contact;
		const FRigidTransform3 AToBTM = AStartTransform.GetRelativeTransform(BTransform);
		const FVec3 LocalDir = BTransform.InverseTransformVectorNoScale(Dir);

		if (!ensure(B.GetType() & ImplicitObjectType::TriangleMesh) || !ensure(!IsInstanced(B.GetType())))
		{
			return FContactPoint();
		}

		Utilities::CastHelper(A, AStartTransform, [&](const auto& ADowncast, const FRigidTransform3& AFullTM)
			{
				FReal TOI, Phi;
				FVec3 Location, Normal;
				int32 FaceIndex = -1;
				Chaos::FVec3 FaceNormal;
				if (B.LowLevelSweepGeomCCD(ADowncast, AToBTM, LocalDir, Length, IgnorePenetration, TargetPenetration, TOI, Phi, Location, Normal, FaceIndex, FaceNormal))
				{
					Contact.ShapeContactPoints[0] = AToBTM.InverseTransformPositionNoScale(Location);
					Contact.ShapeContactPoints[1] = Location;
					Contact.ShapeContactNormal = Normal;
					Contact.Phi = Phi;
					Contact.FaceIndex = FaceIndex;
					OutTOI = TOI;
				}
			});

		return Contact;
	}


	FContactPoint SphereSphereContactPoint(const FSphere& Sphere1, const FRigidTransform3& Sphere1Transform, const FSphere& Sphere2, const FRigidTransform3& Sphere2Transform, const FRealSingle CullDistance)
	{
		FContactPoint Result;

		const FRealSingle R1 = Sphere1.GetRadiusf();
		const FRealSingle R2 = Sphere2.GetRadiusf();

		const FRigidTransform3f Sphere1ToSphere2f = FRigidTransform3f(Sphere1Transform.GetRelativeTransformNoScale(Sphere2Transform));

		const FVec3f Center1 = Sphere1ToSphere2f.TransformPosition(Sphere1.GetCenterf());
		const FVec3f Center2 = Sphere2.GetCenterf();
		const FVec3f Direction = Center1 - Center2;
		const FRealSingle SizeSq = Direction.SizeSquared();
		const FRealSingle CullDistanceSq = FMath::Square(CullDistance + R1 + R2);
		if (SizeSq < CullDistanceSq)
		{
			const FRealSingle Size = FMath::Sqrt(SizeSq);
			const FVec3f Normal = Size > UE_KINDA_SMALL_NUMBER ? Direction / Size : FVec3f(0, 0, 1);
			const FRealSingle NewPhi = Size - (R1 + R2);

			Result.ShapeContactPoints[0] = Sphere1.GetCenterf() - Sphere1ToSphere2f.InverseTransformVector(R1 * Normal);
			Result.ShapeContactPoints[1] = Sphere2.GetCenterf() + R2 * Normal;
			Result.ShapeContactNormal = Normal;
			Result.Phi = NewPhi;
		}
		return Result;
	}

	FContactPoint SpherePlaneContactPoint(const FSphere& Sphere, const FRigidTransform3& SphereTransform, const TPlane<FReal, 3>& Plane, const FRigidTransform3& PlaneTransform)
	{
		FContactPoint Result;

		FReal SphereRadius = Sphere.GetRadiusf();

		FVec3 SpherePosWorld = SphereTransform.TransformPosition(FVec3(Sphere.GetCenterf()));
		FVec3 SpherePosPlane = PlaneTransform.InverseTransformPosition(SpherePosWorld);

		FVec3 NormalPlane;
		FReal Phi = Plane.PhiWithNormal(SpherePosPlane, NormalPlane) - SphereRadius;	// Adding plane's share of padding
		FVec3 NormalWorld = PlaneTransform.TransformVector(NormalPlane);
		FVec3 Location = SpherePosWorld - SphereRadius * NormalWorld;

		Result.ShapeContactPoints[0] = SphereTransform.InverseTransformPosition(Location);
		Result.ShapeContactPoints[1] = PlaneTransform.InverseTransformPosition(Location - Phi * NormalWorld);
		Result.ShapeContactNormal = PlaneTransform.InverseTransformVector(NormalWorld);
		Result.Phi = Phi;

		return Result;
	}

	FContactPoint SphereBoxContactPoint(const FSphere& Sphere, const FRigidTransform3& SphereTransform, const FImplicitBox3& Box, const FRigidTransform3& BoxTransform)
	{
		FContactPoint Result;


		const FVec3 SphereWorld = SphereTransform.TransformPosition(FVec3(Sphere.GetCenterf()));	// World-space sphere pos
		const FVec3 SphereBox = BoxTransform.InverseTransformPosition(SphereWorld);			// Box-space sphere pos

		FVec3 NormalBox;																	// Box-space normal
		FReal PhiToSphereCenter = Box.PhiWithNormal(SphereBox, NormalBox);
		FReal Phi = PhiToSphereCenter - Sphere.GetRadiusf();

		FVec3 NormalWorld = BoxTransform.TransformVectorNoScale(NormalBox);
		FVec3 LocationWorld = SphereWorld - (Sphere.GetRadiusf()) * NormalWorld;

		Result.ShapeContactPoints[0] = SphereTransform.InverseTransformPosition(LocationWorld);
		Result.ShapeContactPoints[1] = BoxTransform.InverseTransformPosition(LocationWorld - Phi * NormalWorld);
		Result.ShapeContactNormal = NormalBox;
		Result.Phi = Phi;
		return Result;
	}

	FContactPoint SphereCapsuleContactPoint(const FSphere& A, const FRigidTransform3& ATransform, const FCapsule& B, const FRigidTransform3& BTransform)
	{
		FContactPoint Result;

		FVector A1 = ATransform.TransformPosition(FVector(A.GetCenterf()));
		FVector B1 = BTransform.TransformPosition(FVector(B.GetX1f()));
		FVector B2 = BTransform.TransformPosition(FVector(B.GetX2f()));
		FVector P2 = FMath::ClosestPointOnSegment(A1, B1, B2);

		FVec3 Delta = P2 - A1;
		FReal DeltaLen = Delta.Size();
		if (DeltaLen > UE_KINDA_SMALL_NUMBER)
		{
			FReal NewPhi = DeltaLen - (A.GetRadiusf() + B.GetRadiusf());
			FVec3 Dir = Delta / DeltaLen;
			FVec3 LocationA = A1 + Dir * A.GetRadiusf();
			FVec3 LocationB = P2 - Dir * B.GetRadiusf();
			FVec3 Normal = -Dir;

			Result.ShapeContactPoints[0] = ATransform.InverseTransformPosition(LocationA);
			Result.ShapeContactPoints[1] = BTransform.InverseTransformPosition(LocationB);
			Result.ShapeContactNormal = BTransform.InverseTransformVector(Normal);
			Result.Phi = NewPhi;
		}

		return Result;
	}

	template <typename TriMeshType>
	FContactPoint SphereTriangleMeshContactPoint(const FSphere& A, const FRigidTransform3& ATransform, const TriMeshType& B, const FRigidTransform3& BTransform, const FReal CullDistance)
	{
		return GJKImplicitContactPoint<FSphere>(FSphere(A), ATransform, B, BTransform, CullDistance);
	}

	template<typename TriMeshType>
	FContactPoint SphereTriangleMeshSweptContactPoint(const FSphere& A, const FRigidTransform3& ATransform, const TriMeshType& B, const FRigidTransform3& BStartTransform, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& TOI)
	{
		if (const TImplicitObjectScaled<FTriangleMeshImplicitObject>* ScaledTriangleMesh = B.template GetObject<const TImplicitObjectScaled<FTriangleMeshImplicitObject>>())
		{
			return GJKImplicitScaledTriMeshSweptContactPoint<FSphere>(A, ATransform, *ScaledTriangleMesh, BStartTransform, Dir, Length, IgnorePenetration, TargetPenetration, TOI);
		}
		else if (const FTriangleMeshImplicitObject* TriangleMesh = B.template GetObject<const FTriangleMeshImplicitObject>())
		{
			return GJKImplicitSweptContactPoint(FSphere(A), ATransform, *TriangleMesh, BStartTransform, Dir, Length, IgnorePenetration, TargetPenetration, TOI);
		}

		ensure(false);
		return FContactPoint();
	}

	FContactPoint BoxHeightFieldContactPoint(const FImplicitBox3& A, const FRigidTransform3& ATransform, const FHeightField& B, const FRigidTransform3& BTransform, const FReal CullDistance)
	{
		return GJKImplicitContactPoint<FImplicitBox3>(A, ATransform, B, BTransform, CullDistance);
	}

	template <typename TriMeshType>
	FContactPoint BoxTriangleMeshContactPoint(const FImplicitBox3& A, const FRigidTransform3& ATransform, const TriMeshType& B, const FRigidTransform3& BTransform, const FReal CullDistance)
	{
		return GJKImplicitContactPoint<TBox<FReal, 3>>(A, ATransform, B, BTransform, CullDistance);
	}

	FContactPoint SphereHeightFieldContactPoint(const FSphere& A, const FRigidTransform3& ATransform, const FHeightField& B, const FRigidTransform3& BTransform, const FReal CullDistance)
	{
		return GJKImplicitContactPoint<FSphere>(FSphere(A), ATransform, B, BTransform, CullDistance);
	}

	FContactPoint CapsuleHeightFieldContactPoint(const FCapsule& A, const FRigidTransform3& ATransform, const FHeightField& B, const FRigidTransform3& BTransform, const FReal CullDistance)
	{
		CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_Collisions_CapsuleHeightFieldContactPoint, ConstraintsDetailedStats);
		return GJKImplicitContactPoint<FCapsule>(A, ATransform, B, BTransform, CullDistance);
	}

	template <typename TriMeshType>
	FContactPoint CapsuleTriangleMeshContactPoint(const FCapsule& A, const FRigidTransform3& ATransform, const TriMeshType& B, const FRigidTransform3& BTransform, const FReal CullDistance)
	{
		CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_Collisions_CapsuleTriangleMeshContactPoint, ConstraintsDetailedStats);
		return GJKImplicitContactPoint<FCapsule>(A, ATransform, B, BTransform, CullDistance);
	}

	template <typename TriMeshType>
	FContactPoint CapsuleTriangleMeshSweptContactPoint(const FCapsule& A, const FRigidTransform3& ATransform, const TriMeshType& B, const FRigidTransform3& BStartTransform, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& TOI)
	{
		CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_Collisions_CapsuleTriangleMeshSweptContactPoint, ConstraintsDetailedStats);
		if (const TImplicitObjectScaled<FTriangleMeshImplicitObject>* ScaledTriangleMesh = B.template GetObject<const TImplicitObjectScaled<FTriangleMeshImplicitObject>>())
		{
			return GJKImplicitScaledTriMeshSweptContactPoint<FCapsule>(A, ATransform, *ScaledTriangleMesh, BStartTransform, Dir, Length, IgnorePenetration, TargetPenetration, TOI);
		}
		else if (const FTriangleMeshImplicitObject* TriangleMesh = B.template GetObject<const FTriangleMeshImplicitObject>())
		{
			return GJKImplicitSweptContactPoint(A, ATransform, *TriangleMesh, BStartTransform, Dir, Length, IgnorePenetration, TargetPenetration, TOI);
		}

		ensure(false);
		return FContactPoint();
	}

	FContactPoint ConvexHeightFieldContactPoint(const FImplicitObject& A, const FRigidTransform3& ATransform, const FHeightField& B, const FRigidTransform3& BTransform, const FReal CullDistance)
	{
		CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_Collisions_ConvexHeightFieldContactPoint, ConstraintsDetailedStats);
		return GJKImplicitContactPoint<FConvex>(A, ATransform, B, BTransform, CullDistance);
	}

	FContactPoint ConvexTriangleMeshContactPoint(const FImplicitObject& A, const FRigidTransform3& ATransform, const FImplicitObject& B, const FRigidTransform3& BTransform, const FReal CullDistance)
	{
		CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_Collisions_ConvexTriangleMeshContactPoint, ConstraintsDetailedStats);

		// Call GJK with the concrete trimesh type (scaled, instanced, raw)
		return Utilities::CastWrapped<FTriangleMeshImplicitObject>(B, 
			[&](auto BConcretePtr)
			{
				check(BConcretePtr != nullptr);
				return GJKImplicitContactPoint<FConvex>(A, ATransform, *BConcretePtr, BTransform, CullDistance);
			});
	}

	template <typename TriMeshType>
	FContactPoint ConvexTriangleMeshSweptContactPoint(const FImplicitObject& A, const FRigidTransform3& ATransform, const TriMeshType& B, const FRigidTransform3& BStartTransform, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& TOI)
	{
		CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_Collisions_ConvexTriangleMeshSweptContactPoint, ConstraintsDetailedStats);
		if (const TImplicitObjectScaled<FTriangleMeshImplicitObject>* ScaledTriangleMesh = B.template GetObject<const TImplicitObjectScaled<FTriangleMeshImplicitObject>>())
		{
			return GJKImplicitScaledTriMeshSweptContactPoint<FConvex>(A, ATransform, *ScaledTriangleMesh, BStartTransform, Dir, Length, IgnorePenetration, TargetPenetration, TOI);
		}
		else if (const FTriangleMeshImplicitObject* TriangleMesh = B.template GetObject<const FTriangleMeshImplicitObject>())
		{
			return GJKImplicitSweptContactPoint(A, ATransform, *TriangleMesh, BStartTransform, Dir, Length, IgnorePenetration, TargetPenetration, TOI);
		}

		ensure(false);
		return FContactPoint();
	}

	FContactPoint CapsuleCapsuleContactPoint(const FCapsule& A, const FRigidTransform3& ATransform, const FCapsule& B, const FRigidTransform3& BTransform)
	{
		FContactPointf Result;

		const FRigidTransform3f CapsuleAToCapsuleBf = FRigidTransform3f(ATransform.GetRelativeTransformNoScale(BTransform));

		const FVector3f A1 = CapsuleAToCapsuleBf.TransformPosition(A.GetX1f());
		const FVector3f A2 = CapsuleAToCapsuleBf.TransformPosition(A.GetX2f());
		const FVector3f B1 = B.GetX1f();
		const FVector3f B2 = B.GetX2f();
		FVector3f P1, P2;
		FMath::SegmentDistToSegmentSafe(A1, A2, B1, B2, P1, P2);

		const FVec3f Delta = P2 - P1;
		const FRealSingle DeltaLen = Delta.Size();
		if (DeltaLen > UE_KINDA_SMALL_NUMBER)
		{
			const FRealSingle NewPhi = DeltaLen - (A.GetRadiusf() + B.GetRadiusf());
			const FVec3f Dir = Delta / DeltaLen;
			const FVec3f Normal = -Dir;
			const FVec3f LocationA = P1 + Dir * A.GetRadiusf();
			const FVec3f LocationB = P2 - Dir * B.GetRadiusf();

			Result.ShapeContactPoints[0] = CapsuleAToCapsuleBf.InverseTransformPositionNoScale(LocationA);
			Result.ShapeContactPoints[1] = LocationB;
			Result.ShapeContactNormal = Normal;
			Result.Phi = NewPhi;
		}

		return Result;
	}

	FContactPoint CapsuleBoxContactPoint(const FCapsule& A, const FRigidTransform3& ATransform, const FImplicitBox3& B, const FRigidTransform3& BTransform, const FVec3& InitialDir)
	{
		return GJKContactPoint(A, ATransform, B, BTransform, InitialDir);
	}


	// Template  Instantiations
	template FContactPoint GJKImplicitSweptContactPoint<FHeightField>(const FImplicitObject& A, const FRigidTransform3& AStartTransform, const FHeightField& B, const FRigidTransform3& BTransform, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& TOI);

	template FContactPoint SphereTriangleMeshSweptContactPoint<TImplicitObjectScaled<class Chaos::FTriangleMeshImplicitObject, 1>>(const FSphere& A, const FRigidTransform3& ATransform, const TImplicitObjectScaled<class Chaos::FTriangleMeshImplicitObject, 1>& B, const FRigidTransform3& BStartTransform, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& TOI);
	template FContactPoint SphereTriangleMeshSweptContactPoint<FTriangleMeshImplicitObject>(const FSphere& A, const FRigidTransform3& ATransform, const FTriangleMeshImplicitObject& B, const FRigidTransform3& BStartTransform, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& TOI);
	template FContactPoint CapsuleTriangleMeshSweptContactPoint<TImplicitObjectScaled<class Chaos::FTriangleMeshImplicitObject, 1>>(const FCapsule& A, const FRigidTransform3& ATransform, const TImplicitObjectScaled<class Chaos::FTriangleMeshImplicitObject, 1>& B, const FRigidTransform3& BStartTransform, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& TOI);
	template FContactPoint CapsuleTriangleMeshSweptContactPoint<FTriangleMeshImplicitObject>(const FCapsule& A, const FRigidTransform3& ATransform, const FTriangleMeshImplicitObject& B, const FRigidTransform3& BStartTransform, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& TOI);
	template FContactPoint ConvexTriangleMeshSweptContactPoint<TImplicitObjectScaled<class Chaos::FTriangleMeshImplicitObject, 1>>(const FImplicitObject& A, const FRigidTransform3& ATransform, const TImplicitObjectScaled<class Chaos::FTriangleMeshImplicitObject, 1>& B, const FRigidTransform3& BStartTransform, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& TOI);
	template FContactPoint ConvexTriangleMeshSweptContactPoint<FTriangleMeshImplicitObject>(const FImplicitObject& A, const FRigidTransform3& ATransform, const FTriangleMeshImplicitObject& B, const FRigidTransform3& BStartTransform, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& TOI);
	template FContactPoint BoxTriangleMeshContactPoint<TImplicitObjectScaled<class Chaos::FTriangleMeshImplicitObject, 1>>(const FImplicitBox3& A, const FRigidTransform3& ATransform, const TImplicitObjectScaled<class Chaos::FTriangleMeshImplicitObject, 1>& B, const FRigidTransform3& BTransform, const FReal CullDistance);
	template FContactPoint BoxTriangleMeshContactPoint<FTriangleMeshImplicitObject>(const FImplicitBox3& A, const FRigidTransform3& ATransform, const FTriangleMeshImplicitObject& B, const FRigidTransform3& BTransform, const FReal CullDistance);
	template FContactPoint SphereTriangleMeshContactPoint<TImplicitObjectScaled<class Chaos::FTriangleMeshImplicitObject, 1>>(const FSphere& A, const FRigidTransform3& ATransform, const TImplicitObjectScaled<class Chaos::FTriangleMeshImplicitObject, 1>& B, const FRigidTransform3& BTransform, const FReal CullDistance);
	template FContactPoint SphereTriangleMeshContactPoint<FTriangleMeshImplicitObject>(const FSphere& A, const FRigidTransform3& ATransform, const FTriangleMeshImplicitObject& B, const FRigidTransform3& BTransform, const FReal CullDistance);
	template FContactPoint CapsuleTriangleMeshContactPoint<TImplicitObjectScaled<class Chaos::FTriangleMeshImplicitObject, 1>>(const FCapsule& A, const FRigidTransform3& ATransform, const TImplicitObjectScaled<class Chaos::FTriangleMeshImplicitObject, 1>& B, const FRigidTransform3& BTransform, const FReal CullDistance);
	template FContactPoint CapsuleTriangleMeshContactPoint<FTriangleMeshImplicitObject>(const FCapsule& A, const FRigidTransform3& ATransform, const FTriangleMeshImplicitObject& B, const FRigidTransform3& BTransform, const FReal CullDistance);
}
