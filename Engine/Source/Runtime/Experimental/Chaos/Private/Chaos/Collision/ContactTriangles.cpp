// Copyright Epic Games, Inc.All Rights Reserved.
#include "Chaos/Collision/ContactTriangles.h"
#include "Chaos/DebugDrawQueue.h"

namespace Chaos
{
	namespace CVars
	{
		extern int32 ChaosSolverDebugDrawMeshContacts;
	}
	extern bool bChaos_Collision_EnableEdgePrune;
	extern bool bChaos_Collision_EnableLargeMeshManifolds;
	extern FRealSingle Chaos_Collision_MeshContactNormalThreshold;
	extern bool bChaos_Collision_MeshManifoldSortByDistance;


	void FContactTriangleCollector::SetNumContacts(const int32 NumContacts)
	{
		TriangleContactPoints.SetNum(NumContacts, EAllowShrinking::No);
		TriangleContactPointDatas.SetNum(NumContacts, EAllowShrinking::No);
	}

	void FContactTriangleCollector::DisableContact(const int32 ContactIndex)
	{
		// Flag the contact as disabled. A subsequent call to RemoveDisabledContacts will repack the array
		// by removing all disabled contacts, without re-ordering
		TriangleContactPointDatas[ContactIndex].SetDisabled();
		++NumDisabledTriangleContactPoints;
	}

	void FContactTriangleCollector::RemoveDisabledContacts()
	{
		// Skip if we have not disabled any contacts
		if (NumDisabledTriangleContactPoints == 0)
		{
			return;
		}

		// Re-pack the contact point array without re-ordering
		const int32 NumContactPoints = TriangleContactPoints.Num();
		int32 DestContactIndex = 0;
		int32 SrcContactIndex = 0;
		while (SrcContactIndex < NumContactPoints)
		{
			if (!TriangleContactPointDatas[SrcContactIndex].IsEnabled())
			{
				// Inner loop to find the next enabled point index to use as the copy source
				while (++SrcContactIndex < NumContactPoints)
				{
					if (TriangleContactPointDatas[SrcContactIndex].IsEnabled())
					{
						break;
					}
				}

				if (SrcContactIndex == NumContactPoints)
				{
					break;
				}
			}

			// If we have removed elements, copy source to dest
			if (DestContactIndex != SrcContactIndex)
			{
				TriangleContactPointDatas[DestContactIndex] = TriangleContactPointDatas[SrcContactIndex];
				TriangleContactPoints[DestContactIndex] = TriangleContactPoints[SrcContactIndex];
			}

			++DestContactIndex;
			++SrcContactIndex;
		}

		// Clip the array to the enabled set
		TriangleContactPointDatas.SetNum(DestContactIndex, EAllowShrinking::No);
		TriangleContactPoints.SetNum(DestContactIndex, EAllowShrinking::No);
		NumDisabledTriangleContactPoints = 0;
	}

	int32 FContactTriangleCollector::AddTriangle(const FTriangle& Triangle, const int32 TriangleIndex, const int32 VertexIndex0, const int32 VertexIndex1, const int32 VertexIndex2)
	{
		const int32 ContactTriangleIndex = ContactTriangles.AddUninitialized();
		FContactTriangle& ContactTriangle = ContactTriangles[ContactTriangleIndex];
		ContactTriangle.Vertices[0] = Triangle.GetVertex(0);
		ContactTriangle.Vertices[1] = Triangle.GetVertex(1);
		ContactTriangle.Vertices[2] = Triangle.GetVertex(2);
		ContactTriangle.FaceNormal = Triangle.GetNormal();
		ContactTriangle.VertexIndices[0] = VertexIndex0;
		ContactTriangle.VertexIndices[1] = VertexIndex1;
		ContactTriangle.VertexIndices[2] = VertexIndex2;
		ContactTriangle.TriangleIndex = TriangleIndex;
		return ContactTriangleIndex;
	}

	// Find the other triangle in the list that shares the specified edge with the specified triangle
	int32 FContactTriangleCollector::GetOtherTriangleIndexSharingEdge(const int32 KnownTriangleIndex, const int32 EdgeVertexIndexA, const int32 EdgeVertexIndexB) const
	{
		check(EdgeVertexIndexA != INDEX_NONE);
		check(EdgeVertexIndexA != EdgeVertexIndexB);

		for (int32 OtherTriangleIndex = 0; OtherTriangleIndex < ContactTriangles.Num(); ++OtherTriangleIndex)
		{
			if (OtherTriangleIndex != KnownTriangleIndex)
			{
				const FContactTriangle& OtherTriangle = ContactTriangles[OtherTriangleIndex];
				if (OtherTriangle.HasVertexIndex(EdgeVertexIndexA))
				{
					if (EdgeVertexIndexB == INDEX_NONE)
					{
						// @todo(chaos): this is just the first triangle using the vertex, there will be at least 3 faces sharing a vertex and could be more...we should handle that
						return OtherTriangleIndex;
					}

					if (OtherTriangle.HasVertexIndex(EdgeVertexIndexB))
					{
						// This triangle also uses the edge
						return OtherTriangleIndex;
					}
				}
			}
		}

		return INDEX_NONE;
	}

	int32 FContactTriangleCollector::GetNextTriangleIndexSharingVertex(const int32 LastContactTriangleIndex, const int32 VertexIndex) const
	{
		check(VertexIndex != INDEX_NONE);
		for (int32 OtherTriangleIndex = LastContactTriangleIndex + 1; OtherTriangleIndex < ContactTriangles.Num(); ++OtherTriangleIndex)
		{
			const FContactTriangle& OtherTriangle = ContactTriangles[OtherTriangleIndex];
			if (OtherTriangle.HasVertexIndex(VertexIndex))
			{
				return OtherTriangleIndex;
			}
		}

		return INDEX_NONE;
	}

	void FContactTriangleCollector::BuildContactFeatureSets()
	{
		const int32 NumContactPoints = TriangleContactPoints.Num();

		// @todo(chaos): use sorted array and binary search rather than TSet (which won't hash well for edge IDs)
		ContactEdges.Reserve(3 * NumContactPoints);
		ContactVertices.Reserve(3 * NumContactPoints);

		for (int32 ContactIndex = 0; ContactIndex < NumContactPoints; ++ContactIndex)
		{
			FTriangleContactPointData& ContactPointData = TriangleContactPointDatas[ContactIndex];
			const FContactPoint& ContactPoint = TriangleContactPoints[ContactIndex];
			const FContactTriangle& ContactTriangle = ContactTriangles[ContactPointData.GetTriangleIndex()];

			// We only use actually penetrating contacts when determining which faces and edges we hit, otherwise
			// we may reject contacts that are actually required based on non-contacts.
			const bool bUseForPruning = (ContactPoint.Phi < 0);

			if (ContactPoint.ContactType == EContactPointType::VertexPlane)
			{
				if (bUseForPruning)
				{
					// Since we collided with this face, we cannot collide with these edges or vertices
					ContactEdges.Add(FContactEdgeID(ContactTriangle.VertexIndices[0], ContactTriangle.VertexIndices[1]));
					ContactEdges.Add(FContactEdgeID(ContactTriangle.VertexIndices[1], ContactTriangle.VertexIndices[2]));
					ContactEdges.Add(FContactEdgeID(ContactTriangle.VertexIndices[2], ContactTriangle.VertexIndices[0]));
					ContactVertices.Add(FContactVertexID(ContactTriangle.VertexIndices[0]));
					ContactVertices.Add(FContactVertexID(ContactTriangle.VertexIndices[1]));
					ContactVertices.Add(FContactVertexID(ContactTriangle.VertexIndices[2]));
				}
			}
			else if ((ContactPoint.ContactType == EContactPointType::EdgeEdge) || (ContactPoint.ContactType == EContactPointType::PlaneVertex))
			{
				int32 VertexIndexA, VertexIndexB;
				FVec3 VertexA, VertexB;
				if (ContactTriangle.GetEdgeVerticesAtPosition(ContactPoint.ShapeContactPoints[1], VertexA, VertexB, VertexIndexA, VertexIndexB, FReal(0.01)))
				{
					if ((VertexIndexA != INDEX_NONE) && (VertexIndexB != INDEX_NONE))
					{
						if (bUseForPruning)
						{
							// Since we collided with this edge, we cannot also collide with these verts
							ContactVertices.Add(FContactVertexID(VertexIndexA));
							ContactVertices.Add(FContactVertexID(VertexIndexB));
						}

						// Store the edge ID so we can see if any other faces add it to the don'tcollide list
						ContactPointData.SetEdgeID(VertexIndexA, VertexIndexB);
					}
					else if (VertexIndexA != INDEX_NONE)
					{
						// Store the vertex ID so we can see if any other faces or edges add it to the don'tcollide list
						ContactPointData.SetVertexID(VertexIndexA);
					}
				}
			}
		}
	}

	void FContactTriangleCollector::ProcessContacts(const FRigidTransform3& MeshToConvexTransform)
	{
		// Build the feature set from the full list of contacts before pruning
		BuildContactFeatureSets();

		DebugDrawContactPoints(FColor::White, 0.2);

		// Remove edge contacts that should not be possible because we are also colliding with a face that used the edge.
		if ((TriangleContactPoints.Num() > 1))
		{
			const bool bPruneEdges = true;
			const bool bPruneVertices = !bChaos_Collision_EnableLargeMeshManifolds;	// @todo(chaos): fix - when set, removes too many interior contacts
			PruneEdgeAndVertexContactPoints(bPruneEdges, bPruneVertices);
		}

		DebugDrawContactPoints(FColor::Cyan, 0.3);

		// Remove contacts that should not be able to occur if we are a one-sided mesh
		// NOTE: does not require sorted contacts, and does not maintain array order
		if (bOneSidedCollision)
		{
			PruneInfacingContactPoints();
		}

		DebugDrawContactPoints(FColor::Purple, 0.5);

		// Remove close contacts
		if (TriangleContactPoints.Num() > 1)
		{
			PruneUnnecessaryContactPoints();
		}

		DebugDrawContactPoints(FColor::Yellow, 0.7);

		if (!bChaos_Collision_EnableLargeMeshManifolds)
		{
			// Reduce to 4 contact points from here
			ReduceManifoldContactPointsTriangeMesh();
		}

		DebugDrawContactPoints(FColor::Green, 0.9);

		// Fix contact normals for edge and vertex collisions
		if (TriangleContactPoints.Num() > 0)
		{
			FixInvalidNormalContactPoints();
		}

		DebugDrawContactPoints(FColor::Red, 1.0);

		FinalizeContacts(MeshToConvexTransform);
	}

	void FContactTriangleCollector::SortContactPointsForPruning()
	{
		// Repack the array
		RemoveDisabledContacts();

		// Sort TriangleContactPoints and TriangleContactPointDatas for pruning
		if (TriangleContactPoints.Num() > 1)
		{
			TArray<int32> SortedIndices;
			SortedIndices.SetNumUninitialized(TriangleContactPoints.Num());
			for (int32 Index = 0; Index < TriangleContactPoints.Num(); ++Index)
			{
				SortedIndices[Index] = Index;
			}

			// Sort by depth
			std::sort(
				&SortedIndices[0],
				&SortedIndices[0] + SortedIndices.Num(),
				[this](const int32& L, const int32& R)
				{
					return TriangleContactPoints[L].Phi < TriangleContactPoints[R].Phi;
				}
			);

			TArray<FContactPoint> SortedContactPoints;
			TArray<FTriangleContactPointData> SortedContactPointDatas;
			SortedContactPoints.SetNum(TriangleContactPoints.Num());
			SortedContactPointDatas.SetNum(TriangleContactPoints.Num());
			for (int32 Index = 0; Index < TriangleContactPoints.Num(); ++Index)
			{
				SortedContactPoints[Index] = TriangleContactPoints[SortedIndices[Index]];
				SortedContactPointDatas[Index] = TriangleContactPointDatas[SortedIndices[Index]];
			}
			Swap(TriangleContactPoints, SortedContactPoints);
			Swap(TriangleContactPointDatas, SortedContactPointDatas);
		}
	}

	// Sort contacts on a shape pair in the order we like to solve them.
	// NOTE: This relies on the enum order of EContactPointType.
	void FContactTriangleCollector::SortContactPointsForSolving()
	{
		// Repack the array
		RemoveDisabledContacts();

		// Sort TriangleContactPoints in solver preferred order, but ignore TriangleContactPointDatas
		// NOTE: This should only be called at the end of the pruning proxess when we no longer care 
		// about TriangleContactPointDatas.
		if (TriangleContactPoints.Num() > 1)
		{
			// Reset metadata to ensure we don't access it again
			TriangleContactPointDatas.Reset();

			if (bChaos_Collision_MeshManifoldSortByDistance)
			{
				// Sort contact points by distance from the center of mass (RxN) so that points closer to the center of
				// mass are solved first. This produces better solver results for low iterations because, if we were to 
				// solve the distant points first, we would get extra rotation applied. 
				//
				// E.g., consider a box landing on an inclined plane with 5 contact points biassed toward one side. 
				//
				// -------------------
				// |                 |
				// |                 |
				// *-*-*-*-*----------
				//
				// Solving this left to right would result in extra clockwise rotation after 1 iteration. A subsequent
				// iteration would partially correct the problem.
				//
				TArray<TPair<FReal, int32>> SortKeyValues;
				SortKeyValues.SetNumUninitialized(TriangleContactPoints.Num());
				for (int32 ContactIndex = 0; ContactIndex < TriangleContactPoints.Num(); ++ContactIndex)
				{
					const FContactPoint& ContactPoint = TriangleContactPoints[ContactIndex];
					const FVec3 DeltaTangent = ContactPoint.ShapeContactPoints[1] - FVec3::DotProduct(ContactPoint.ShapeContactPoints[1], ContactPoint.ShapeContactNormal) * ContactPoint.ShapeContactNormal;
					const FReal DeltaTangentLenSq = DeltaTangent.SizeSquared();
					SortKeyValues[ContactIndex] = { DeltaTangentLenSq, ContactIndex };
				}

				std::sort(
					&SortKeyValues[0],
					&SortKeyValues[0] + SortKeyValues.Num(),
					[](const TPair<FReal, int32>& L, const TPair<FReal, int32>& R)
					{
						return L.Key < R.Key;
					}
				);

				TArray<FContactPoint> SortedContactPoints;
				SortedContactPoints.SetNumUninitialized(TriangleContactPoints.Num());
				for (int32 ContactIndex = 0; ContactIndex < TriangleContactPoints.Num(); ++ContactIndex)
				{
					SortedContactPoints[ContactIndex] = TriangleContactPoints[SortKeyValues[ContactIndex].Value];
				}
				Swap(SortedContactPoints, TriangleContactPoints);
			}
			else
			{
				// Sort by contact type (in enum order) then depth (deepest first)
				std::sort(
					&TriangleContactPoints[0],
					&TriangleContactPoints[0] + TriangleContactPoints.Num(),
					[](const FContactPoint& L, const FContactPoint& R)
					{
						if (L.ContactType == R.ContactType)
						{
							return L.Phi < R.Phi;
						}
						return L.ContactType < R.ContactType;
					}
				);
			}
		}
	}

	void FContactTriangleCollector::PruneEdgeAndVertexContactPoints(const bool bPruneEdges, const bool bPruneVertices)
	{
		if (!bChaos_Collision_EnableEdgePrune)
		{
			return;
		}

		// Remove contacts that are invalid edge or vertex contacts
		// - vertex collisions if we have an edge collision that includes the vertex
		// - edge collisions if we have a face collision that includes the edge
		for (int32 ContactIndex = TriangleContactPointDatas.Num() - 1; ContactIndex >= 0; --ContactIndex)
		{
			FTriangleContactPointData& ContactPointData = TriangleContactPointDatas[ContactIndex];
			if (!ContactPointData.IsEnabled())
			{
				continue;
			}

			bool bRemoveContact = false;
			if (bPruneVertices && ContactPointData.IsVertex())
			{
				bRemoveContact = ContactVertices.Contains(ContactPointData.GetVertexID());
			}
			else if (bPruneEdges && ContactPointData.IsEdge())
			{
				bRemoveContact = ContactEdges.Contains(ContactPointData.GetEdgeID());
			}

			if (bRemoveContact)
			{
				DisableContact(ContactIndex);
			}
		}
	}

	// Examine all edge and vertex contacts. If the contact normal is invalid (i.e., it points into the prism formed by 
	// infinitly extending the face along the normal) then fix it to be in the valid range.
	void FContactTriangleCollector::FixInvalidNormalContactPoints()
	{
		const FReal ContactDotNormalThreshold = Chaos_Collision_MeshContactNormalThreshold;

		const int32 NumContactPoints = TriangleContactPoints.Num();
		for (int32 ContactIndex = 0; ContactIndex < NumContactPoints; ++ContactIndex)
		{
			FTriangleContactPointData& ContactPointData = TriangleContactPointDatas[ContactIndex];
			FContactPoint& ContactPoint = TriangleContactPoints[ContactIndex];
			if (!ContactPointData.IsEnabled())
			{
				continue;
			}

			// Do we have a collision with the triangle edge or vertex?
			// NOTE: second shape is always the triangle
			const bool bIsTriangleEdgeOrVertex = (ContactPoint.ContactType == EContactPointType::EdgeEdge) || (ContactPoint.ContactType == EContactPointType::PlaneVertex);
			if (bIsTriangleEdgeOrVertex)
			{
				const int32 ContactTriangleIndex = ContactPointData.GetTriangleIndex();
				const FContactTriangle& ContactTriangle = ContactTriangles[ContactTriangleIndex];

				// If the normal points roughly along the triangle normal, it won't be too bad even if it's technically invalid so just leave it
				const FReal ContactDotNormal = FReal(ContactPointData.GetContactNormalDotTriangleNormal());
				if (ContactDotNormal > ContactDotNormalThreshold)
				{
					continue;
				}

				// Find the vertex positions and indices for this contact
				int32 VertexIndexA, VertexIndexB;
				FVec3 VertexA, VertexB;
				if (ContactTriangle.GetEdgeVerticesAtPosition(ContactPoint.ShapeContactPoints[1], VertexA, VertexB, VertexIndexA, VertexIndexB))
				{
					// If VertexIndexA and VertexIndexB are valid we have an edge collision.
					// If only VertexIndexA is valid (VertexIndexB is INDEX_NONE) we have a vertex collision. 
					// @todo(chaos): Add check for valid normals with vertex collisions. This is much harder than the edge case...
					if ((VertexIndexA != INDEX_NONE) && (VertexIndexB != INDEX_NONE))
					{
						// Get the other triangle that shares this edge
						const int32 OtherContactTriangleIndex = GetOtherTriangleIndexSharingEdge(ContactTriangleIndex, VertexIndexA, VertexIndexB);
						if (OtherContactTriangleIndex != INDEX_NONE)
						{
							const FContactTriangle& OtherContactTriangle = ContactTriangles[OtherContactTriangleIndex];
							const FReal MinContactDotNormal = FVec3::DotProduct(OtherContactTriangle.FaceNormal, ContactTriangle.FaceNormal);
							if (ContactDotNormal < MinContactDotNormal)
							{
								// We are outside the valid normal range for this edge
								// Convert the edge collision to a face collision on one of the faces, selected to get the smallest depth
								FVec3 CorrectedContactNormal;
								int32 CorrectedTriangleIndex;
								const FReal OtherContactDotNormal = FVec3::DotProduct(ContactPoint.ShapeContactNormal, OtherContactTriangle.FaceNormal);
								if (ContactDotNormal >= OtherContactDotNormal)
								{
									CorrectedContactNormal = (ContactDotNormal > -SMALL_NUMBER) ? ContactTriangle.FaceNormal : -ContactTriangle.FaceNormal;
									CorrectedTriangleIndex = ContactTriangleIndex;
								}
								else
								{
									CorrectedContactNormal = (OtherContactDotNormal > -SMALL_NUMBER) ? OtherContactTriangle.FaceNormal : -OtherContactTriangle.FaceNormal;
									CorrectedTriangleIndex = OtherContactTriangleIndex;
								}

								// If we had to change the normal by a lot, disable the contacts
								//if (ContactPoint.Phi > 0)
								//{
								//	const FReal CorrectedNormalThreshold = -KINDA_SMALL_NUMBER;	// 90 deg
								//	const FReal CorrectedContactNormalDotContactNormal = FVec3::DotProduct(CorrectedContactNormal, ContactPoint.ShapeContactNormal);
								//	if (CorrectedContactNormalDotContactNormal < CorrectedNormalThreshold)
								//	{
								//		DisableContact(ContactIndex);
								//		continue;
								//	}
								//}

								// NOTE: We keep the contact depth as it is because we know that the depth is a lower-bound. 
								// We have to update the ShapeContactPoint[0] because Phi is actually derived from the positions (the value in ContactPoint is just a cache of current state)
								// @todo(chaos): face selection logic might be better if we knew the contact velocity
								ContactPoint.ShapeContactNormal = CorrectedContactNormal;
								ContactPoint.ShapeContactPoints[0] = ContactPoint.ShapeContactPoints[1] + ContactPoint.Phi * CorrectedContactNormal;
								ContactPointData.SetTriangleIndex(CorrectedTriangleIndex);
							}
						}
					}
					else if (VertexIndexA != INDEX_NONE)
					{
						// If VertexIndexA is valid (VertexIndexB is invalid) we have a vertex collision.
						// Ensure that the contact normal is in a valid range for the vertex based on the
						// triangles that share the vertex.
						for (int32 TriangleIndex = 0; TriangleIndex < ContactTriangles.Num(); ++TriangleIndex)
						{
							const FContactTriangle& OtherContactTriangle = ContactTriangles[TriangleIndex];
							if (OtherContactTriangle.HasVertexIndex(VertexIndexA))
							{
								FVec3 EdgeVertex0, EdgeVertex1;
								if (OtherContactTriangle.GetOtherVertexPositions(VertexIndexA, EdgeVertex0, EdgeVertex1))
								{
									// Does the contact normal point into the infinite prism formed by extruding the triangle along the face normal?
									const FVec3 EdgeDelta0 = EdgeVertex0 - VertexA;
									const FVec3 EdgeDelta1 = EdgeVertex1 - VertexA;
									const FReal EdgeSign0 = FVec3::DotProduct(FVec3::CrossProduct(ContactPoint.ShapeContactNormal, EdgeVertex0 - VertexA), OtherContactTriangle.FaceNormal);
									const FReal EdgeSign1 = FVec3::DotProduct(FVec3::CrossProduct(ContactPoint.ShapeContactNormal, EdgeVertex1 - VertexA), OtherContactTriangle.FaceNormal);
									if (FMath::Sign(EdgeSign0) == FMath::Sign(EdgeSign1))
									{
										const FVec3 Centroid = OtherContactTriangle.GetCentroid();
										const FReal NormalDotCentroid = FVec3::DotProduct(ContactPoint.ShapeContactNormal, Centroid - ContactPoint.ShapeContactPoints[1]);
										if (NormalDotCentroid > 0)
										{
											const FReal OtherContactDotNormal = FVec3::DotProduct(ContactPoint.ShapeContactNormal, OtherContactTriangle.FaceNormal);
											const FVec3 CorrectedContactNormal = (OtherContactDotNormal > -SMALL_NUMBER) ? OtherContactTriangle.FaceNormal : -OtherContactTriangle.FaceNormal;

											// If we had to change the normal by a lot, disable the contacts
											//if (ContactPoint.Phi > 0)
											//{
											//	const FReal CorrectedNormalThreshold = -KINDA_SMALL_NUMBER;	// 90 deg
											//	const FReal CorrectedContactNormalDotContactNormal = FVec3::DotProduct(CorrectedContactNormal, ContactPoint.ShapeContactNormal);
											//	if (CorrectedContactNormalDotContactNormal < CorrectedNormalThreshold)
											//	{
											//		DisableContact(ContactIndex);
											//		break;
											//	}
											//}

											ContactPoint.ShapeContactNormal = CorrectedContactNormal;
											ContactPoint.ShapeContactPoints[0] = ContactPoint.ShapeContactPoints[1] + ContactPoint.Phi * CorrectedContactNormal;
										}
									}
								}
							}
						}
					}
				}
			}
		}
	}

	// Remove contact points that have a normal pointing inthe opposite direction of the face normal
	void FContactTriangleCollector::PruneInfacingContactPoints()
	{
		for (int32 ContactIndex = TriangleContactPointDatas.Num() - 1; ContactIndex >= 0; --ContactIndex)
		{
			FTriangleContactPointData& ContactPointData = TriangleContactPointDatas[ContactIndex];
			const FContactPoint& ContactPoint = TriangleContactPoints[ContactIndex];
			const FContactTriangle& ContactTriangle = ContactTriangles[ContactPointData.GetTriangleIndex()];

			if (ContactPointData.IsEnabled())
			{
				const FReal ContactNormalDotFaceNormal = ContactPointData.GetContactNormalDotTriangleNormal();// FVec3::DotProduct(ContactPoint.ShapeContactNormal, ContactTriangle.FaceNormal);
				if (ContactNormalDotFaceNormal < 0)
				{
					DisableContact(ContactIndex);
				}
			}
		}
	}

	void FContactTriangleCollector::PruneUnnecessaryContactPoints()
	{
		if (TriangleContactPoints.Num() < 2)
		{
			return;
		}

		// Below is O(N^2) so packing contacts is beneficial when there are lots of contacts
		RemoveDisabledContacts();

		const int32 NumContactPoints = TriangleContactPointDatas.Num();

		// Remove points that are very close together. Always keep the one that is least likely to be an edge contact
		// @todo(chaos): Use a grid to remove the O(N^2) loop
		const FReal DistanceToleranceSq = FMath::Square(DistanceTolerance);
		for (int32 ContactIndex0 = 0; ContactIndex0 < NumContactPoints; ++ContactIndex0)
		{
			FTriangleContactPointData& ContactPointData0 = TriangleContactPointDatas[ContactIndex0];
			const FContactPoint& ContactPoint0 = TriangleContactPoints[ContactIndex0];
			const FContactTriangle& ContactTriangle0 = ContactTriangles[ContactPointData0.GetTriangleIndex()];
			if (!ContactPointData0.IsEnabled())
			{
				continue;
			}

			// Remove duplicate contact points (each triangle generates a manifold so the vertex and edge contacts are typically duplicated)
			for (int32 ContactIndex1 = ContactIndex0 + 1; ContactIndex1 < NumContactPoints; ++ContactIndex1)
			{
				FTriangleContactPointData& ContactPointData1 = TriangleContactPointDatas[ContactIndex1];
				const FContactPoint& ContactPoint1 = TriangleContactPoints[ContactIndex1];
				const FContactTriangle& ContactTriangle1 = ContactTriangles[ContactPointData1.GetTriangleIndex()];
				if (!ContactPointData1.IsEnabled())
				{
					continue;
				}

				// Are these two points at the same position? They must be at the same vertex or edge for this to be true.
				// If both points are from the same triangle vertex, we know they are coincident (but normals may be different)
				const bool bIsSameVertex = (ContactPointData0.IsVertex() && ContactPointData1.IsVertex()) && (ContactPointData0.GetVertexID() == ContactPointData1.GetVertexID());
				bool bIsSamePosition = bIsSameVertex;
				if (!bIsSameVertex)
				{
					// Check distance and normal
					// NOTE: ShapeContactPoints[1] is on the triangle
					const FVec3 ContactDelta = ContactPoint0.ShapeContactPoints[1] - ContactPoint1.ShapeContactPoints[1];
					const FReal ContactDeltaSq = ContactDelta.SizeSquared();
					bIsSamePosition = (ContactDeltaSq < DistanceToleranceSq);
				}

				if (bIsSamePosition)
				{
					// Keep the contact with the normal that is most along the triangle face normal. If normals are similar, keep the deepest.
					const FReal NormalDot0 = ContactPointData0.GetContactNormalDotTriangleNormal();// FVec3::DotProduct(ContactPoint0.ShapeContactNormal, ContactTriangle0.FaceNormal);
					const FReal NormalDot1 = ContactPointData1.GetContactNormalDotTriangleNormal();// FVec3::DotProduct(ContactPoint1.ShapeContactNormal, ContactTriangle1.FaceNormal);
					const bool bSameNormal = FMath::IsNearlyEqual(NormalDot0, NormalDot1, FReal(0.01));
					const bool bTake0 = (bSameNormal && (ContactPoint0.Phi < ContactPoint1.Phi)) || (!bSameNormal && (NormalDot0 > NormalDot1));
					if (bTake0)
					{
						DisableContact(ContactIndex1);
						continue;
					}
					else
					{
						DisableContact(ContactIndex0);
						break;
					}
				}
			}
		}
	}

	// Reduce the number of contact points (in place)
	void FContactTriangleCollector::ReduceManifoldContactPointsTriangeMesh()
	{
		// Sort contact points on phi (ascending)
		SortContactPointsForPruning();

		// Make sure we packed the contacts
		check(NumDisabledTriangleContactPoints == 0);

		// NOTE: The above pack and sort may reduce the point count to below our threshold
		if (TriangleContactPoints.Num() <= 4)
		{
			return;
		}

		// Point 1) is the deepest contact point
		// It is already in position

		//
		// @todo(chaos): actually this isn't really good enough. We want to take the outer 4 points for the most stable manifold, but also
		// must take the deepest for correct behaviour. We also have a problem right now where if we have a number of contacts at the same 
		// depth, the "deepest" one is arbitrary and may be a poor choice as the anchor of the manifold building. E.g., Consider 5 contact 
		// points in a X config where the center point is first in the list - this leads to a triangular/unstable manifold.
		// 
		// Maybe we should do this:
		//	- build the manifold using the current process
		//	- revisit point 0 at the end and replace with the best choice for stability if the depth is similar
		// Or this:
		//	- build manifold for best stabilty (largest convex area)
		//	- if manifold does not contain deepest point, replace one point so minimize area reduction
		//

		// Point 2) Find the point with the largest distance to the deepest contact point
		{
			uint32 FarthestPointIndex = 1;
			FReal FarthestPointDistanceSQR = -1.0f;
			for (int32 PointIndex = 1; PointIndex < TriangleContactPoints.Num(); PointIndex++)
			{
				FReal PointAToPointBSizeSQR = (TriangleContactPoints[PointIndex].ShapeContactPoints[1] - TriangleContactPoints[0].ShapeContactPoints[1]).SizeSquared();
				if (PointAToPointBSizeSQR > FarthestPointDistanceSQR)
				{
					FarthestPointIndex = PointIndex;
					FarthestPointDistanceSQR = PointAToPointBSizeSQR;
				}
			}
			// Farthest point will be added now
			Swap(TriangleContactPoints[1], TriangleContactPoints[FarthestPointIndex]);
			Swap(TriangleContactPointDatas[1], TriangleContactPointDatas[FarthestPointIndex]);
		}

		// Point 3) Largest triangle area
		{
			uint32 LargestTrianglePointIndex = 2;
			FReal LargestTrianglePointSignedArea = 0.0f; // This will actually be double the signed area
			FVec3 P0to1 = TriangleContactPoints[1].ShapeContactPoints[1] - TriangleContactPoints[0].ShapeContactPoints[1];
			for (int32 PointIndex = 2; PointIndex < TriangleContactPoints.Num(); PointIndex++)
			{
				FReal TriangleSignedArea = FVec3::DotProduct(FVec3::CrossProduct(P0to1, TriangleContactPoints[PointIndex].ShapeContactPoints[1] - TriangleContactPoints[0].ShapeContactPoints[1]), TriangleContactPoints[0].ShapeContactNormal);
				if (FMath::Abs(TriangleSignedArea) > FMath::Abs(LargestTrianglePointSignedArea))
				{
					LargestTrianglePointIndex = PointIndex;
					LargestTrianglePointSignedArea = TriangleSignedArea;
				}
			}
			// Point causing the largest triangle will be added now
			Swap(TriangleContactPoints[2], TriangleContactPoints[LargestTrianglePointIndex]);
			Swap(TriangleContactPointDatas[2], TriangleContactPointDatas[LargestTrianglePointIndex]);

			// Ensure the winding order is consistent
			if (LargestTrianglePointSignedArea < 0)
			{
				Swap(TriangleContactPoints[0], TriangleContactPoints[1]);
				Swap(TriangleContactPointDatas[0], TriangleContactPointDatas[1]);
			}
		}

		// Point 4) Find the largest triangle connecting with our current triangle
		{
			uint32 LargestTrianglePointIndex = 3;
			FReal LargestPositiveTrianglePointSignedArea = 0.0f;
			for (int32 PointIndex = 3; PointIndex < TriangleContactPoints.Num(); PointIndex++)
			{
				for (uint32 EdgeIndex = 0; EdgeIndex < 3; EdgeIndex++)
				{
					FReal TriangleSignedArea = FVec3::DotProduct(FVec3::CrossProduct(TriangleContactPoints[PointIndex].ShapeContactPoints[1] - TriangleContactPoints[EdgeIndex].ShapeContactPoints[1], TriangleContactPoints[(EdgeIndex + 1) % 3].ShapeContactPoints[1] - TriangleContactPoints[EdgeIndex].ShapeContactPoints[1]), TriangleContactPoints[0].ShapeContactNormal);
					if (TriangleSignedArea > LargestPositiveTrianglePointSignedArea)
					{
						LargestTrianglePointIndex = PointIndex;
						LargestPositiveTrianglePointSignedArea = TriangleSignedArea;
					}
				}
			}
			// Point causing the largest positive triangle area will be added now
			Swap(TriangleContactPoints[3], TriangleContactPoints[LargestTrianglePointIndex]);
			Swap(TriangleContactPointDatas[3], TriangleContactPointDatas[LargestTrianglePointIndex]);
		}

		// Revisit Point 0 - if we have a choice that increases area at similar depth, use it

		// Will end up with 4 points
		SetNumContacts(4);
	}

	void FContactTriangleCollector::FinalizeContacts(const FRigidTransform3& MeshToConvexTransform)
	{
		// Remove any disabled contacts and sort
		SortContactPointsForSolving();

		// Transform contact data back into shape-local space
		for (int32 ContactIndex = 0; ContactIndex < TriangleContactPoints.Num(); ++ContactIndex)
		{
			FContactPoint& ContactPoint = TriangleContactPoints[ContactIndex];

			ContactPoint.ShapeContactPoints[1] = MeshToConvexTransform.InverseTransformPositionNoScale(ContactPoint.ShapeContactPoints[1]);
			ContactPoint.ShapeContactNormal = MeshToConvexTransform.InverseTransformVectorNoScale(ContactPoint.ShapeContactNormal);
		}
	}

	void FContactTriangleCollector::DebugDrawContactPoints(const FColor& Color, const FReal LineScale)
	{
#if CHAOS_DEBUG_DRAW
		if (CVars::ChaosSolverDebugDrawMeshContacts && FDebugDrawQueue::GetInstance().IsDebugDrawingEnabled())
		{
			TArray<bool> bTriangleDrawn;
			bTriangleDrawn.SetNumZeroed(ContactTriangles.Num());

			const FReal Duration = 0;
			const int32 DrawPriority = 10;
			for (int32 ContactIndex = 0; ContactIndex < TriangleContactPointDatas.Num(); ++ContactIndex)
			{
				const FTriangleContactPointData& ContactPointData = TriangleContactPointDatas[ContactIndex];
				const FContactPoint& ContactPoint = TriangleContactPoints[ContactIndex];
				if (!ContactPointData.IsEnabled())
				{
					continue;
				}

				const FVec3 P0 = ConvexTransform.TransformPosition(ContactPoint.ShapeContactPoints[0]);
				const FVec3 P1 = ConvexTransform.TransformPosition(ContactPoint.ShapeContactPoints[1]);
				const FVec3 N = ConvexTransform.TransformVectorNoScale(ContactPoint.ShapeContactNormal);

				// Draw the normal from the triangle face
				FDebugDrawQueue::GetInstance().DrawDebugLine(P1, P1 + LineScale * FReal(50) * N, Color, false, FRealSingle(Duration), DrawPriority, 2 * FRealSingle(LineScale));

				// Draw a thin black line connecting the two contact points (triangle face to convex surface)
				//FDebugDrawQueue::GetInstance().DrawDebugLine(P0, P1, FColor::Black, false, FRealSingle(Duration), DrawPriority, 0.5f * FRealSingle(LineThickness));

				if (!bTriangleDrawn[ContactPointData.GetTriangleIndex()])
				{
					FContactTriangle& Triangle = ContactTriangles[ContactPointData.GetTriangleIndex()];
					const FVec3 V0 = ConvexTransform.TransformPosition(Triangle.Vertices[0]);
					const FVec3 V1 = ConvexTransform.TransformPosition(Triangle.Vertices[1]);
					const FVec3 V2 = ConvexTransform.TransformPosition(Triangle.Vertices[2]);
					FDebugDrawQueue::GetInstance().DrawDebugLine(V0, V1, FColor::Silver, false, FRealSingle(Duration), DrawPriority, FRealSingle(LineScale));
					FDebugDrawQueue::GetInstance().DrawDebugLine(V1, V2, FColor::Silver, false, FRealSingle(Duration), DrawPriority, FRealSingle(LineScale));
					FDebugDrawQueue::GetInstance().DrawDebugLine(V2, V0, FColor::Silver, false, FRealSingle(Duration), DrawPriority, FRealSingle(LineScale));
					bTriangleDrawn[ContactPointData.GetTriangleIndex()] = true;
				}
			}
		}
#endif
	}

}