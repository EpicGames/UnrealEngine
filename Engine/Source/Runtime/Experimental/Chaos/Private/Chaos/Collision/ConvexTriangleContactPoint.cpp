// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/Collision/ConvexTriangleContactPoint.h"
#include "Chaos/Collision/ContactPoint.h"
#include "Chaos/Collision/ContactTriangles.h"
#include "Chaos/Collision/ConvexContactPoint.h"
#include "Chaos/Collision/ConvexContactPointUtilities.h"
#include "Chaos/Collision/SATConvexTriangle.h"
#include "Chaos/CollisionOneShotManifolds.h"
#include "Chaos/Convex.h"
#include "Chaos/DebugDrawQueue.h"
#include "Chaos/SAT.h"
#include "Chaos/Triangle.h"
#include "Chaos/Utilities.h"
#include "Misc/MemStack.h"

namespace Chaos
{
	extern FRealSingle Chaos_Collision_GJKEpsilon;
	extern FRealSingle Chaos_Collision_EPAEpsilon;
	extern bool bChaos_Collision_UseConvexTriangleGJKSAT;

	// Clip the vertices of a triangle to a face of a convex, using some arbitrary vector as the clipping axis
	// (the axis is assumed to not be parallel to the convex face surface).
	template <typename ConvexImplicitType>
	TArrayView<FVec3> ClipTriangleToConvex(
		const FTriangle& Triangle,
		const ConvexImplicitType& Convex,
		const int32 ConvexPlaneIndex,
		const FVec3& Axis,
		TArrayView<FVec3> VertexBuffer1,
		TArrayView<FVec3> VertexBuffer2)
	{
		check(VertexBuffer1.Num() == VertexBuffer2.Num());
		check(VertexBuffer1.Num() >= 3);

		// Start with the triangle vertices
		int32 ContactPointCount = 0;
		VertexBuffer1[0] = Triangle.GetVertex(0);
		VertexBuffer1[1] = Triangle.GetVertex(1);
		VertexBuffer1[2] = Triangle.GetVertex(2);
		ContactPointCount = 3;

		// Now clip against all planes that belong to the convex plane's, edges
		// Note winding order matters here, and we have to handle negative scales
		const FReal ConvexWindingOrder = Convex.GetWindingOrder();
		const int32 ConvexFaceVerticesNum = Convex.NumPlaneVertices(ConvexPlaneIndex);
		int32 ClippingPlaneCount = ConvexFaceVerticesNum;
		FVec3 PrevPoint = Convex.GetVertex(Convex.GetPlaneVertex(ConvexPlaneIndex, ClippingPlaneCount - 1));
		for (int32 ClippingPlaneIndex = 0; (ClippingPlaneIndex < ClippingPlaneCount) && (ContactPointCount > 1); ++ClippingPlaneIndex)
		{
			const FVec3 CurrentPoint = Convex.GetVertex(Convex.GetPlaneVertex(ConvexPlaneIndex, ClippingPlaneIndex));

			// Convex edge clipping plane
			// NOTE: Plane is not normalized, but the length cancels out in all clip operations
			const FVec3 ClippingPlaneNormal = ConvexWindingOrder * FVec3::CrossProduct(Axis, PrevPoint - CurrentPoint);
			if (ClippingPlaneNormal.SizeSquared() > UE_SMALL_NUMBER)
			{
				ContactPointCount = Collisions::ClipVerticesAgainstPlane(VertexBuffer1.GetData(), VertexBuffer2.GetData(), ContactPointCount, VertexBuffer1.Num(), ClippingPlaneNormal, FVec3::DotProduct(CurrentPoint, ClippingPlaneNormal));
				Swap(VertexBuffer1, VertexBuffer2); // VertexBuffer1 will now point to the latest
			}

			PrevPoint = CurrentPoint;
		}

		return TArrayView<FVec3>(VertexBuffer1.GetData(), ContactPointCount);
	}

	// Clip the vertices of a convex to a face of a triangle
	template <typename ConvexImplicitType>
	TArrayView<FVec3> ClipConvexToTriangle(
		const ConvexImplicitType& Convex,
		const int32 ConvexPlaneIndex,
		const FTriangle& Triangle,
		const FVec3& TriangleN,
		TArrayView<FVec3> VertexBuffer1,
		TArrayView<FVec3> VertexBuffer2)
	{
		check(VertexBuffer1.Num() == VertexBuffer2.Num());

		// Populate the clipped vertices with the convex face vertices
		const FReal ConvexWindingOrder = Convex.GetWindingOrder();
		int32 ContactPointCount = 0;
		const int32 ConvexFaceVerticesNum = Convex.NumPlaneVertices(ConvexPlaneIndex);
		ContactPointCount = FMath::Min(ConvexFaceVerticesNum, VertexBuffer1.Num()); // Number of face vertices
		for (int32 VertexIndex = 0; VertexIndex < ContactPointCount; ++VertexIndex)
		{
			const int32 BufferIndex = (ConvexWindingOrder >= 0) ? VertexIndex : ContactPointCount - VertexIndex - 1;
			VertexBuffer1[BufferIndex] = Convex.GetVertex(Convex.GetPlaneVertex(ConvexPlaneIndex, VertexIndex));
		}

		// Now clip against all planes that belong to the reference plane's, edges
		// Note winding order matters here, and we have to handle negative scales
		int32 ClippingPlaneCount = 3;
		FVec3 PrevPoint = Triangle.GetVertex(2);
		for (int32 ClippingPlaneIndex = 0; (ClippingPlaneIndex < ClippingPlaneCount) && (ContactPointCount > 1); ++ClippingPlaneIndex)
		{
			const FVec3 CurrentPoint = Triangle.GetVertex(ClippingPlaneIndex);

			// Triangle edge clipping plane
			// NOTE: Plane is not normalized, but the length cancels out in all clip operations
			const FVec3 ClippingPlaneNormal = FVec3::CrossProduct(TriangleN, PrevPoint - CurrentPoint);
			if (ClippingPlaneNormal.SizeSquared() > UE_SMALL_NUMBER)
			{
				ContactPointCount = Collisions::ClipVerticesAgainstPlane(VertexBuffer1.GetData(), VertexBuffer2.GetData(), ContactPointCount, VertexBuffer1.Num(), ClippingPlaneNormal, FVec3::DotProduct(CurrentPoint, ClippingPlaneNormal));
				Swap(VertexBuffer1, VertexBuffer2); // VertexBuffer1 will now point to the latest
			}

			PrevPoint = CurrentPoint;
		}

		return TArrayView<FVec3>(VertexBuffer1.GetData(), ContactPointCount);
	}

	// Triangle is in (possibly scaled) Convex space
	template <typename ConvexType>
	void ConstructConvexTriangleOneShotManifold2(const ConvexType& Convex, const FTriangle& Triangle, const FReal CullDistance, FContactPointManifold& OutContactPoints)
	{
		const FReal NormalTolerance = FReal(1.e-8);
		const FReal NormalToleranceSq = NormalTolerance * NormalTolerance;
		const FReal InvalidPhi = std::numeric_limits<FReal>::lowest();

		// Triangle (same space as convex)
		const FVec3 TriN = Triangle.GetNormal();
		const FVec3 TriC = Triangle.GetCentroid();

		//
		// SAT: Triangle face vs Convex verts
		//
		
		// We store the convex vertex distances to the triangle face for use in the edge-edge culling
		FMemMark Mark(FMemStack::Get());
		TArray<FReal, TMemStackAllocator<alignof(FReal)>> ConvexVertexDs;
		ConvexVertexDs.SetNum(Convex.NumVertices());
		TArrayView<FReal> ConvexVertexDsView = MakeArrayView(ConvexVertexDs);

		// Project the convex onto the triangle normal, with distances relative to the triangle plane
		FReal TriPlaneDMin, TriPlaneDMax;
		int32 ConvexVertexIndexMin, ConvexVertexIndexMax;
		Private::ProjectOntoAxis(Convex, TriN, TriC, TriPlaneDMin, TriPlaneDMax, ConvexVertexIndexMin, ConvexVertexIndexMax, &ConvexVertexDsView);

		// Distance culling
		if (TriPlaneDMin > CullDistance)
		{
			// Outside triangle face and separated by more than CullDistance
			return;
		}
		if (TriPlaneDMax < FReal(0))
		{
			// Inside triangle face (single-sided collision)
			return;
		}

		//
		// SAT: Convex faces vs triangle verts
		//

		// For each convex plane, project the Triangle onto the convex plane normal
		// and reject if the separation is more than cull distance.
		FVec3 ConvexPlaneN = FVec3(0);
		FVec3 ConvexPlaneX = FVec3(0);
		FReal ConvexPlaneDMin = InvalidPhi;
		int32 ConvexPlaneIndexMin = INDEX_NONE;
		for (int32 PlaneIndex = 0; PlaneIndex < Convex.NumPlanes(); ++PlaneIndex)
		{
			FVec3 ConN, ConX;
			Convex.GetPlaneNX(PlaneIndex, ConN, ConX);

			FReal DMin, DMax;
			int32 IndexMin, IndexMax;
			Private::ProjectOntoAxis(Triangle, ConN, ConX, DMin, DMax, IndexMin, IndexMax);

			// Distance culling
			// @todo(chaos): Cull against the projected convex hull, not just the outward face 
			// (we can store the most-distance vertex for each face with the convex to avoid actually having to project)
			if (DMin > CullDistance)
			{
				// Separated by more than CullDistance
				return;
			}

			if (DMin > ConvexPlaneDMin)
			{
				ConvexPlaneN = ConN;
				ConvexPlaneX = ConX;
				ConvexPlaneDMin = DMin;
				ConvexPlaneIndexMin = PlaneIndex;
			}
		}

		//
		// SAT: Convex edges vs triangle edges
		//

		// Calculate the distance of triangle each edge to the convex separating plane
		FReal TriVertexConvexDMin0 = FVec3::DotProduct(Triangle.GetVertex(0) - ConvexPlaneX, ConvexPlaneN);
		FReal TriVertexConvexDMin1 = FVec3::DotProduct(Triangle.GetVertex(1) - ConvexPlaneX, ConvexPlaneN);
		FReal TriVertexConvexDMin2 = FVec3::DotProduct(Triangle.GetVertex(2) - ConvexPlaneX, ConvexPlaneN);
		FReal TriEdgeConvexDMin[3] =
		{
			FMath::Min(TriVertexConvexDMin2, TriVertexConvexDMin0),
			FMath::Min(TriVertexConvexDMin0, TriVertexConvexDMin1),
			FMath::Min(TriVertexConvexDMin1, TriVertexConvexDMin2),
		};
		
		FReal ConvexWinding = Convex.GetWindingOrder();
		FVec3 EdgeEdgeN = FVec3(0);
		FReal EdgeEdgeDMin = InvalidPhi;
		int32 ConvexEdgeIndexMin = INDEX_NONE;
		int32 TriEdgeIndexMin = INDEX_NONE;
		for (int32 ConvexEdgeLoopIndex = 0; ConvexEdgeLoopIndex < Convex.NumEdges(); ++ConvexEdgeLoopIndex)
		{
			// Handle reverse winding for negative scaled convexes. Loop over edges in reverse order, and reverse edge vertex order
			const int32 ConvexEdgeIndex = (ConvexWinding >= 0) ? ConvexEdgeLoopIndex : Convex.NumEdges() - ConvexEdgeLoopIndex - 1;
			const int32 ConvexEdgeVIndex0 = (ConvexWinding >= 0) ? 0 : 1;
			const int32 ConvexEdgeVIndex1 = (ConvexWinding >= 0) ? 1 : 0;

			// Skip convex edges beyond CullDistance of the triangle face
			const int32 ConvexEdgeVertexIndex0 = Convex.GetEdgeVertex(ConvexEdgeIndex, ConvexEdgeVIndex0);
			const int32 ConvexEdgeVertexIndex1 = Convex.GetEdgeVertex(ConvexEdgeIndex, ConvexEdgeVIndex1);
			const FReal FaceConvexD0 = ConvexVertexDs[ConvexEdgeVertexIndex0];
			const FReal FaceConvexD1 = ConvexVertexDs[ConvexEdgeVertexIndex1];
			if ((FaceConvexD0 > CullDistance) && (FaceConvexD1 > CullDistance))
			{
				continue;
			}

			// Convex edge vertices
			const FVec3 ConvexEdgeV0 = Convex.GetVertex(ConvexEdgeVertexIndex0);
			const FVec3 ConvexEdgeV1 = Convex.GetVertex(ConvexEdgeVertexIndex1);

			// Convex planes that form the edge
			const int32 ConvexEdgePlaneIndexA = Convex.GetEdgePlane(ConvexEdgeIndex, 0);
			const int32 ConvexEdgePlaneIndexB = Convex.GetEdgePlane(ConvexEdgeIndex, 1);
			const FVec3 ConvexEdgePlaneNormalA = Convex.GetPlane(ConvexEdgePlaneIndexA).Normal();
			const FVec3 ConvexEdgePlaneNormalB = Convex.GetPlane(ConvexEdgePlaneIndexB).Normal();

			for (int32 TriEdgeIndex = 0; TriEdgeIndex < 3; ++TriEdgeIndex)
			{
				// Skip triangle edges beyond cull distance of the convex separating face
				if (TriEdgeConvexDMin[TriEdgeIndex] > CullDistance)
				{
					continue;
				}

				// Triangle edge vertices
				const FVec3& TriEdgeV0 = Triangle.GetVertex(TriEdgeIndex);
				const FVec3& TriEdgeV1 = (TriEdgeIndex == 2) ? Triangle.GetVertex(0) : Triangle.GetVertex(TriEdgeIndex + 1);

				// Skip edge pairs that do not contribute to the Minkowski Sum surface
				// NOTE: This relies on the ordering of the edge planes from above. 
				// I.e., we require Sign(ConvexEdgePlaneNormalA x ConvexEdgePlaneNormalB) == Sign(ConvexEdgeV1 - ConvexEdgeV0)
				// Also note that we must pass the negated triangle normal in
				if (!Private::IsOnMinkowskiSumConvexTriangle(ConvexEdgePlaneNormalA, ConvexEdgePlaneNormalB, ConvexEdgeV1 - ConvexEdgeV0, -TriN, TriEdgeV1 - TriEdgeV0))
				{
					continue;
				}

				// Separating axis
				// NOTE: Not normalized at this stage. We perform the projection against the
				// non-normalized axis and defer the square root until we know we need it
				FVec3 Axis = FVec3::CrossProduct(ConvexEdgeV1 - ConvexEdgeV0, TriEdgeV1 - TriEdgeV0);
				const FReal AxisLenSq = Axis.SizeSquared();

				// Pick consistent axis direction: away from triangle (we want a signed distance)
				const FReal Sign = FVec3::DotProduct(TriEdgeV0 - TriC, Axis);
				if (Sign < FReal(0))
				{
					Axis = -Axis;
				}

				const FReal ScaledSeparation = FVec3::DotProduct(ConvexEdgeV0 - TriEdgeV0, Axis);

				// Check cull distance on projected segments
				// Comparing square distances scaled by axis length to defer square root (keep the sign)
				const FReal ScaledSeparationSq = ScaledSeparation * FMath::Abs(ScaledSeparation);
				const FReal ScaledCullDistanceSq = FMath::Square(CullDistance) * AxisLenSq;
				if (ScaledSeparationSq > ScaledCullDistanceSq)
				{
					return;
				}

				// Comparing square distances scaled by axis length to defer square root (keep the sign)
				const FReal ScaledEdgeEdgeDMinSq = EdgeEdgeDMin * FMath::Abs(EdgeEdgeDMin) * AxisLenSq;
				if (ScaledSeparationSq > ScaledEdgeEdgeDMinSq)
				{
					// Now we need to know the actual separation and axis
					const FReal AxisInvLen = FMath::InvSqrt(AxisLenSq);
					EdgeEdgeDMin = ScaledSeparation * AxisInvLen;
					EdgeEdgeN = Axis * AxisInvLen;
					ConvexEdgeIndexMin = ConvexEdgeIndex;
					TriEdgeIndexMin = TriEdgeIndex;
				}
			}
		}

		// Determine which of the features we want to use
		// NOTE: we rely on the fact that all valid Phi values are greater than InvalidPhi here
		const FReal TriFaceBias = FReal(1.e-2);	// Prevent flip-flop on near-parallel cases
		EContactPointType ContactType = EContactPointType::Unknown;
		if ((TriPlaneDMin != InvalidPhi) && (TriPlaneDMin + TriFaceBias > ConvexPlaneDMin) && (TriPlaneDMin + TriFaceBias > EdgeEdgeDMin))
		{
			// Tri plane is the shallowest penetration
			ContactType = EContactPointType::VertexPlane;
		}
		else if ((ConvexPlaneDMin != InvalidPhi) && (ConvexPlaneDMin > EdgeEdgeDMin))
		{
			// Convex plane is the shallowest penetration
			ContactType = EContactPointType::PlaneVertex;
		}
		else if (EdgeEdgeDMin != InvalidPhi)
		{
			// Edge-edge is the shallowest penetration
			ContactType = EContactPointType::EdgeEdge;
		}
		else
		{
			// No valid features (should not happen - TriPlaneDMin should always be valid)
			return;
		}

		// Determine the best features to use for this collision
		FVec3 SeparatingAxis, ClipAxis;
		bool bClipConvexToTriangle;
		bool bClipToFaceNormal;
		if (ContactType == EContactPointType::VertexPlane)
		{
			// Triangle face contact - clip the convex vertices to the triangle
			
			bClipConvexToTriangle = true;
			bClipToFaceNormal = true;

			// The triangle normal is the separating axis
			SeparatingAxis = TriN;
			if (FVec3::DotProduct(SeparatingAxis, Convex.GetCenterOfMass() - TriC) < FReal(0))
			{
				SeparatingAxis = -SeparatingAxis;
			}

			ClipAxis = TriN;

			// Find the convex face most opposing the separating axis			
			ConvexPlaneIndexMin = Convex.GetMostOpposingPlane(TriN);		// @todo(chaos): should use the known vertex index
			Convex.GetPlaneNX(ConvexPlaneIndexMin, ConvexPlaneN, ConvexPlaneX);

		}
		else if (ContactType == EContactPointType::PlaneVertex)
		{
			// Convex face contact - clip the triangle to the convex face
			bClipConvexToTriangle = false;
			bClipToFaceNormal = true;

			// The convex face is the separating axis, but it must point from the triangle to the convex
			SeparatingAxis = ConvexPlaneN;
			if (FVec3::DotProduct(SeparatingAxis, Convex.GetCenterOfMass() - TriC) < FReal(0))
			{
				SeparatingAxis = -SeparatingAxis;
			}

			ClipAxis = ConvexPlaneN;
		}
		else if (ContactType == EContactPointType::EdgeEdge) //-V547
		{
			// Edge-edge contact - clip triangle vs convex or vice-versa based on most opposing normals

			// The separating axis must point from the triangle to the convex
			SeparatingAxis = EdgeEdgeN;
			if (FVec3::DotProduct(SeparatingAxis, Convex.GetCenterOfMass() - TriC) < FReal(0))
			{
				SeparatingAxis = -SeparatingAxis;
			}

			// Find the convex face most opposing the separating axis			
			ConvexPlaneIndexMin = Convex.GetMostOpposingPlane(SeparatingAxis);	// @todo(chaos): should use the known edge index
			Convex.GetPlaneNX(ConvexPlaneIndexMin, ConvexPlaneN, ConvexPlaneX);

			// Decide whether to clip against the triangle or the convex
			ClipAxis = SeparatingAxis;
			const FReal TriNDotAxis = FVec3::DotProduct(TriN, SeparatingAxis);
			const FReal ConvexNDotAxis = FVec3::DotProduct(ConvexPlaneN, SeparatingAxis);
			if (FMath::Abs(TriNDotAxis) > FMath::Abs(ConvexNDotAxis))
			{
				bClipConvexToTriangle = true;
				if (FVec3::DotProduct(ClipAxis, TriN) < 0)
				{
					ClipAxis = -ClipAxis;
				}
			}
			else
			{
				bClipConvexToTriangle = false;
				if (FVec3::DotProduct(ClipAxis, ConvexPlaneN) < 0)
				{
					ClipAxis = -ClipAxis;
				}
			}
			bClipToFaceNormal = false;
		}
		else
		{
			return;
		}

		// @todo(chaos): scratch or stack allocation of clipped vertex buffers
		const int32 MaxContactPointCount = 32;
		TCArray<FVec3, MaxContactPointCount> ClippedVertexBuffers0 = TCArray<FVec3, MaxContactPointCount>::MakeFull();
		TCArray<FVec3, MaxContactPointCount> ClippedVertexBuffers1 = TCArray<FVec3, MaxContactPointCount>::MakeFull();

		// @todo(chaos): 2D Clip
		TArrayView<FVec3> ClippedVertices;
		if (bClipConvexToTriangle)
		{
			ClippedVertices = ClipConvexToTriangle(Convex, ConvexPlaneIndexMin, Triangle, ClipAxis, MakeArrayView(ClippedVertexBuffers0), MakeArrayView(ClippedVertexBuffers1));
		}
		else
		{
			ClippedVertices = ClipTriangleToConvex(Triangle, Convex, ConvexPlaneIndexMin, ClipAxis, MakeArrayView(ClippedVertexBuffers0), MakeArrayView(ClippedVertexBuffers1));
		}

		// Reduce number of contacts to the maximum allowed
		if (ClippedVertices.Num() > 4)
		{
			const FRotation3 RotateSeperationToZ = FRotation3::FromRotatedVector(SeparatingAxis, FVec3(0.0f, 0.0f, 1.0f));
			for (int32 ContactPointIndex = 0; ContactPointIndex < ClippedVertices.Num(); ++ContactPointIndex)
			{
				ClippedVertices[ContactPointIndex] = RotateSeperationToZ * ClippedVertices[ContactPointIndex];
			}

			const int32 ReducedContactPointCount = Collisions::ReduceManifoldContactPoints(ClippedVertices.GetData(), ClippedVertices.Num());
			ClippedVertices = MakeArrayView(ClippedVertices.GetData(), ReducedContactPointCount);

			for (int32 ContactPointIndex = 0; ContactPointIndex < ClippedVertices.Num(); ++ContactPointIndex)
			{
				ClippedVertices[ContactPointIndex] = RotateSeperationToZ.Inverse() * ClippedVertices[ContactPointIndex];
			}
		}

		// Add the clipped points to the contact list
		const auto& AddContact = [&CullDistance, &ContactType, &SeparatingAxis, &OutContactPoints](const FVec3& ConvexX, const FVec3 TriX, const FReal Distance)
		{
			if (Distance < CullDistance)
			{
				FContactPoint& ContactPoint = OutContactPoints[OutContactPoints.AddUninitialized()];
				ContactPoint.ShapeContactPoints[0] = ConvexX;
				ContactPoint.ShapeContactPoints[1] = TriX;
				ContactPoint.ShapeContactNormal = SeparatingAxis;
				ContactPoint.Phi = Distance;
				ContactPoint.ContactType = ContactType;
				ContactPoint.FaceIndex = INDEX_NONE;
			}
		};

		check(ClippedVertices.Num() <= OutContactPoints.Max());
		if (bClipConvexToTriangle && bClipToFaceNormal)
		{
			// Clipped points are on the convex, and we clipped to the triangle face along its normal
			for (int32 ContactIndex = 0; ContactIndex < ClippedVertices.Num(); ++ContactIndex)
			{
				const FVec3& ConvexX = ClippedVertices[ContactIndex];
				const FReal Distance = FVec3::DotProduct(ConvexX - TriC, SeparatingAxis);
				const FVec3 TriX = ConvexX - Distance * SeparatingAxis;

				AddContact(ConvexX, TriX, Distance);
			}
		}
		else if (bClipConvexToTriangle && !bClipToFaceNormal)
		{
			// Clipped points are on the convex, and we clipped to the triangle face, but not along its normal
			for (int32 ContactIndex = 0; ContactIndex < ClippedVertices.Num(); ++ContactIndex)
			{
				const FVec3& ConvexX = ClippedVertices[ContactIndex];
				const FReal IntersectDenom = FVec3::DotProduct(SeparatingAxis, TriN);
				check(FMath::Abs(IntersectDenom) > UE_SMALL_NUMBER);	// Guaranteed based on axis selection above
				const FReal Distance = FVec3::DotProduct(ConvexX - TriC, TriN) / IntersectDenom;
				const FVec3 TriX = ConvexX - Distance * SeparatingAxis;

				AddContact(ConvexX, TriX, Distance);
			}
		}
		else if (!bClipConvexToTriangle && bClipToFaceNormal)
		{
			// Clipped points are on the triangle, and we clipped to the convex face along its normal
			for (int32 ContactIndex = 0; ContactIndex < ClippedVertices.Num(); ++ContactIndex)
			{
				const FVec3& TriX = ClippedVertices[ContactIndex];
				const FReal Distance = FVec3::DotProduct(ConvexPlaneX - TriX, SeparatingAxis);
				const FVec3 ConvexX = TriX + Distance * SeparatingAxis;

				AddContact(ConvexX, TriX, Distance);
			}
		}
		else if (!bClipConvexToTriangle && !bClipToFaceNormal)
		{
			// Clipped points are on the triangle, and we clipped to the convex face, but not along its normal
			for (int32 ContactIndex = 0; ContactIndex < ClippedVertices.Num(); ++ContactIndex)
			{
				const FVec3& TriX = ClippedVertices[ContactIndex];
				const FReal IntersectDenom = FVec3::DotProduct(SeparatingAxis, ConvexPlaneN);
				check(FMath::Abs(IntersectDenom) > UE_SMALL_NUMBER);	// Guaranteed based on axis selection above
				const FReal Distance = FVec3::DotProduct(ConvexPlaneX - TriX, ConvexPlaneN) / IntersectDenom;
				const FVec3 ConvexX = TriX + Distance * SeparatingAxis;

				AddContact(ConvexX, TriX, Distance);
			}
		}
	}

	template void ConstructConvexTriangleOneShotManifold2(
		const FImplicitConvex3& Convex, 
		const FTriangle& Triangle,
		const FReal CullDistance, 
		FContactPointManifold& OutContactPoints);

	template void ConstructConvexTriangleOneShotManifold2(
		const TImplicitObjectInstanced<FImplicitConvex3>& Convex,
		const FTriangle& Triangle,
		const FReal CullDistance,
		FContactPointManifold& OutContactPoints);

	template void ConstructConvexTriangleOneShotManifold2(
		const TImplicitObjectScaled<FImplicitConvex3>& Convex,
		const FTriangle& Triangle,
		const FReal CullDistance,
		FContactPointManifold& OutContactPoints);

	template void ConstructConvexTriangleOneShotManifold2(
		const FImplicitBox3& Convex,
		const FTriangle& Triangle,
		const FReal CullDistance,
		FContactPointManifold& OutContactPoints);

	template  void ConstructConvexTriangleOneShotManifold2(
		const TImplicitObjectScaled<FImplicitBox3>& Convex,
		const FTriangle& Triangle,
		const FReal CullDistance,
		FContactPointManifold& OutContactPoints);

	template  void ConstructConvexTriangleOneShotManifold2(
		const TImplicitObjectInstanced<FImplicitBox3>& Convex,
		const FTriangle& Triangle,
		const FReal CullDistance,
		FContactPointManifold& OutContactPoints);

} // Chaos

///////////////////////////////////////////////////////////////////////////////
///////////////////////////////////////////////////////////////////////////////
///////////////////////////////////////////////////////////////////////////////
///////////////////////////////////////////////////////////////////////////////
///////////////////////////////////////////////////////////////////////////////

namespace Chaos::Private
{
	// Generate a contact manifold between a convex and a triangle, given the closest feature (i.e., single Contact point)
	template <typename ConvexType>
	void ConvexTriangleManifoldFromContact(const ConvexType& Convex, const FTriangle& Triangle, const FVec3& TriangleNormal, const FConvexContactPoint& Contact, const FReal CullDistance, FContactPointManifold& OutManifold)
	{
		// Convex plane
		const int32 ConvexPlaneIndex = Contact.Features[0].PlaneIndex;
		check(ConvexPlaneIndex != INDEX_NONE);
		FVec3 ConvexPlaneN, ConvexPlaneX;
		Convex.GetPlaneNX(ConvexPlaneIndex, ConvexPlaneN, ConvexPlaneX);

		// Triangle plane
		check(Contact.Features[1].PlaneIndex == 0);
		const FVec3 TriN = TriangleNormal;
		const FVec3 TriC = Triangle.GetCentroid();

		const FVec3& SeparatingAxis = Contact.ShapeContactNormal;

		// Decide whether we want to clip the tri against the convex face or vice-versa
		bool bClipConvexToTriangle = false;
		FVec3 ClipAxis = SeparatingAxis;
		const FReal TriNDotAxis = FVec3::DotProduct(TriN, ClipAxis);
		const FReal ConvexNDotAxis = FVec3::DotProduct(ConvexPlaneN, ClipAxis);
		if (FMath::Abs(TriNDotAxis) > FMath::Abs(ConvexNDotAxis))
		{
			bClipConvexToTriangle = true;
			if (TriNDotAxis < 0)
			{
				ClipAxis = -ClipAxis;
			}
		}
		else
		{
			bClipConvexToTriangle = false;
			if (ConvexNDotAxis < 0)
			{
				ClipAxis = -ClipAxis;
			}
		}

		// @todo(chaos): scratch or stack allocation of clipped vertex buffers
		// @todo(chaos): 2D Clip
		const int32 MaxContactPointCount = 32;
		TCArray<FVec3, MaxContactPointCount> ClippedVertexBuffers0 = TCArray<FVec3, MaxContactPointCount>::MakeFull();
		TCArray<FVec3, MaxContactPointCount> ClippedVertexBuffers1 = TCArray<FVec3, MaxContactPointCount>::MakeFull();
		TArrayView<FVec3> ClippedVertices;
		if (bClipConvexToTriangle)
		{
			ClippedVertices = ClipConvexToTriangle(Convex, ConvexPlaneIndex, Triangle, ClipAxis, MakeArrayView(ClippedVertexBuffers0), MakeArrayView(ClippedVertexBuffers1));
		}
		else
		{
			ClippedVertices = ClipTriangleToConvex(Triangle, Convex, ConvexPlaneIndex, ClipAxis, MakeArrayView(ClippedVertexBuffers0), MakeArrayView(ClippedVertexBuffers1));
		}

		// Reduce number of contacts to the maximum allowed
		if (ClippedVertices.Num() > 4)
		{
			const FRotation3 RotateSeperationToZ = FRotation3::FromRotatedVector(SeparatingAxis, FVec3(0.0f, 0.0f, 1.0f));
			for (int32 ContactPointIndex = 0; ContactPointIndex < ClippedVertices.Num(); ++ContactPointIndex)
			{
				ClippedVertices[ContactPointIndex] = RotateSeperationToZ * ClippedVertices[ContactPointIndex];
			}

			const int32 ReducedContactPointCount = Collisions::ReduceManifoldContactPoints(ClippedVertices.GetData(), ClippedVertices.Num());
			ClippedVertices = MakeArrayView(ClippedVertices.GetData(), ReducedContactPointCount);

			for (int32 ContactPointIndex = 0; ContactPointIndex < ClippedVertices.Num(); ++ContactPointIndex)
			{
				ClippedVertices[ContactPointIndex] = RotateSeperationToZ.Inverse() * ClippedVertices[ContactPointIndex];
			}
		}
		check(ClippedVertices.Num() <= OutManifold.Max());

		const EContactPointType ContactType = Contact.GetContactPointType();

		// Add the clipped points to the contact list
		const auto& AddContact = [&CullDistance, &ContactType, &SeparatingAxis, &OutManifold](const FVec3& ConvexX, const FVec3 TriX, const FReal Distance)
		{
			if (Distance < CullDistance)
			{
				FContactPoint& ContactPoint = OutManifold[OutManifold.AddUninitialized()];
				ContactPoint.ShapeContactPoints[0] = ConvexX;
				ContactPoint.ShapeContactPoints[1] = TriX;
				ContactPoint.ShapeContactNormal = SeparatingAxis;
				ContactPoint.Phi = Distance;
				ContactPoint.ContactType = ContactType;
				ContactPoint.FaceIndex = INDEX_NONE;
			}
		};

		// If the convex center projects inside the triangle and inside the convex face, add it as the first contact. This helps with solver convergence.
		// @todo(chaos): ideally this would be the particle center of mass - we could pass that in.
		// @todo(chaos): also need to check that the points projects inside the convex face
		//const FVec3 ConvexCenterBarycentric = ToBarycentric(Convex.GetCenterOfMass(), Triangle.GetVertex(0), Triangle.GetVertex(1), Triangle.GetVertex(2));
		//if ((ConvexCenterBarycentric.X >= 0) && (ConvexCenterBarycentric.Y >= 0) && (ConvexCenterBarycentric.Z >= 0))
		//{
		//	const FVec3 TriX = FromBarycentric(ConvexCenterBarycentric, Triangle.GetVertex(0), Triangle.GetVertex(1), Triangle.GetVertex(2));
		//	const FReal ConvexNDotTriN = FVec3::DotProduct(ConvexPlaneN, TriN);
		//	if (ConvexNDotTriN < UE_KINDA_SMALL_NUMBER)
		//	{
		//		const FReal Distance = FVec3::DotProduct(ConvexPlaneX - TriX, ConvexPlaneN) / ConvexNDotTriN;
		//		const FVec3 ConvexX = TriX + Distance * TriN;
		//		AddContact(ConvexX, TriX, Distance);
		//	}
		//}

		if (bClipConvexToTriangle)
		{
			// Clipped points are on the convex, and we clipped to the triangle face
			for (int32 ContactIndex = 0; ContactIndex < ClippedVertices.Num(); ++ContactIndex)
			{
				const FVec3& ConvexX = ClippedVertices[ContactIndex];
				const FReal IntersectDenom = FVec3::DotProduct(SeparatingAxis, TriN);
				check(FMath::Abs(IntersectDenom) > UE_SMALL_NUMBER);	// Guaranteed based on axis selection above
				const FReal Distance = FVec3::DotProduct(ConvexX - TriC, TriN) / IntersectDenom;
				const FVec3 TriX = ConvexX - Distance * SeparatingAxis;

				AddContact(ConvexX, TriX, Distance);
			}
		}
		else
		{
			// Clipped points are on the triangle, and we clipped to the convex face
			for (int32 ContactIndex = 0; ContactIndex < ClippedVertices.Num(); ++ContactIndex)
			{
				const FVec3& TriX = ClippedVertices[ContactIndex];
				const FReal IntersectDenom = FVec3::DotProduct(SeparatingAxis, ConvexPlaneN);
				check(FMath::Abs(IntersectDenom) > UE_SMALL_NUMBER);	// Guaranteed based on axis selection above
				const FReal Distance = FVec3::DotProduct(ConvexPlaneX - TriX, ConvexPlaneN) / IntersectDenom;
				const FVec3 ConvexX = TriX + Distance * SeparatingAxis;

				AddContact(ConvexX, TriX, Distance);
			}
		}
	}

	template void ConvexTriangleManifoldFromContact(
		const FImplicitConvex3& Convex, 
		const FTriangle& Triangle, 
		const FVec3& TriangleNormal, 
		const FConvexContactPoint& Contact,
		const FReal CullDistance, 
		FContactPointManifold& OutManifold);

	template void ConvexTriangleManifoldFromContact(
		const TImplicitObjectInstanced<FImplicitConvex3>& Convex,
		const FTriangle& Triangle, 
		const FVec3& TriangleNormal,
		const FConvexContactPoint& Contact,
		const FReal CullDistance, 
		FContactPointManifold& OutManifold);

	template void ConvexTriangleManifoldFromContact(
		const TImplicitObjectScaled<FImplicitConvex3>& Convex,
		const FTriangle& Triangle,
		const FVec3& TriangleNormal,
		const FConvexContactPoint& Contact,
		const FReal CullDistance,
		FContactPointManifold& OutManifold);

	template void ConvexTriangleManifoldFromContact(
		const FImplicitBox3& Convex,
		const FTriangle& Triangle,
		const FVec3& TriangleNormal,
		const FConvexContactPoint& Contact,
		const FReal CullDistance,
		FContactPointManifold& OutManifold);

	template void ConvexTriangleManifoldFromContact(
		const TImplicitObjectScaled<FImplicitBox3>& Convex,
		const FTriangle& Triangle,
		const FVec3& TriangleNormal,
		const FConvexContactPoint& Contact,
		const FReal CullDistance,
		FContactPointManifold& OutManifold);

	template void ConvexTriangleManifoldFromContact(
		const TImplicitObjectInstanced<FImplicitBox3>& Convex,
		const FTriangle& Triangle,
		const FVec3& TriangleNormal,
		const FConvexContactPoint& Contact,
		const FReal CullDistance,
		FContactPointManifold& OutManifold);


	// Generate a single contact point between a convex and a triangle
	// NOTE: Does not fill in the Feature properties of OutContactPoint (see GetConvexFeature for that)
	template <typename ConvexType>
	bool ConvexTriangleContactPoint(const ConvexType& Convex, const FTriangle& Triangle, const FVec3& TriangleNormal, const FReal CullDistanceSq, FConvexContactPoint& OutContactPoint)
	{
		const FReal GJKEpsilon = Chaos_Collision_GJKEpsilon;
		const FReal EPAEpsilon = Chaos_Collision_EPAEpsilon;
		const TGJKCoreShape<ConvexType> GJKConvex(Convex, Convex.GetMarginf());
		const TGJKShape<FTriangle> GJKTriangle(Triangle);

		// The GJKPenetrationSameSpace version uses EPA which has issues with almost exactly touching shapes. 
		// E.g., see failing unit test EPARealFailures_TouchingBoxTriangle
		if (bChaos_Collision_UseConvexTriangleGJKSAT)
		{
			// Use GJK to find closest features. This will abort if the margin-reduced shapes are closer than GJKEpsilon or overlap
			constexpr FReal InvalidDistance = std::numeric_limits<FReal>::max();
			FReal Distance = InvalidDistance;
			FVec3 ConvexClosest, TriangleClosest, ConvexNormal;

			const FVec3 InitialV = GJKDistanceInitialVFromDirection(GJKConvex, GJKTriangle, Triangle.GetCentroid());

			const EGJKDistanceResult GJKResult = GJKDistance(
				GJKConvex, GJKTriangle, InitialV, 
				Distance, ConvexClosest, TriangleClosest, ConvexNormal, 
				GJKEpsilon);

			// GJK did not provide a result if the margin-reduced shapes overlap, so run SAT to calculate the minimum separating axis
			if (GJKResult == EGJKDistanceResult::DeepContact)
			{
				return Private::SATConvexTriangle(
					Convex, Triangle, TriangleNormal, 
					CullDistanceSq, OutContactPoint);
			}

			// We are separated, or ovelap within marging so fill in the output
			check(Distance != InvalidDistance);
			if (Utilities::SignedSquare(Distance) < CullDistanceSq)
			{
				OutContactPoint.ShapeContactPoints[0] = ConvexClosest;
				OutContactPoint.ShapeContactPoints[1] = TriangleClosest;
				OutContactPoint.ShapeContactNormal = -ConvexNormal;
				OutContactPoint.Phi = Distance;
				return true;
			}
		}
		else
		{
			FReal UnusedMaxMarginDelta = FReal(0);
			int32 ConvexVertexIndex = INDEX_NONE;
			int32 TriangleVertexIndex = INDEX_NONE;
			FReal Penetration;
			FVec3 ConvexClosest, TriangleClosest, ConvexNormal;
			FVec3 InitialGJKDir = FVec3(1, 0, 0);

			const bool bHaveContact = GJKPenetrationSameSpace(
				GJKConvex, GJKTriangle, 
				Penetration, ConvexClosest, TriangleClosest, ConvexNormal, ConvexVertexIndex, TriangleVertexIndex, UnusedMaxMarginDelta, 
				InitialGJKDir, GJKEpsilon, EPAEpsilon);

			if (bHaveContact && (FMath::Square(Penetration) < CullDistanceSq))
			{
				OutContactPoint.ShapeContactPoints[0] = ConvexClosest;
				OutContactPoint.ShapeContactPoints[1] = TriangleClosest;
				OutContactPoint.ShapeContactNormal = -ConvexNormal;
				OutContactPoint.Phi = -Penetration;
				return true;
			}
		}

		return false;
	}

	template bool ConvexTriangleContactPoint(
		const FImplicitConvex3& Convex,
		const FTriangle& Triangle,
		const FVec3& TriangleNormal,
		const FReal CullDistance,
		FConvexContactPoint& OutContactPoint);

	template bool ConvexTriangleContactPoint(
		const TImplicitObjectInstanced<FImplicitConvex3>& Convex,
		const FTriangle& Triangle,
		const FVec3& TriangleNormal,
		const FReal CullDistance,
		FConvexContactPoint& OutContactPoint);

	template bool ConvexTriangleContactPoint(
		const TImplicitObjectScaled<FImplicitConvex3>& Convex,
		const FTriangle& Triangle,
		const FVec3& TriangleNormal,
		const FReal CullDistance,
		FConvexContactPoint& OutContactPoint);

	template bool ConvexTriangleContactPoint(
		const FImplicitBox3& Convex,
		const FTriangle& Triangle,
		const FVec3& TriangleNormal,
		const FReal CullDistance,
		FConvexContactPoint& OutContactPoint);

	template bool ConvexTriangleContactPoint(
		const TImplicitObjectScaled<FImplicitBox3>& Convex,
		const FTriangle& Triangle,
		const FVec3& TriangleNormal,
		const FReal CullDistance,
		FConvexContactPoint& OutContactPoint);

	template bool ConvexTriangleContactPoint(
		const TImplicitObjectInstanced<FImplicitBox3>& Convex,
		const FTriangle& Triangle,
		const FVec3& TriangleNormal,
		const FReal CullDistance,
		FConvexContactPoint& OutContactPoint);


	// Generate the contact point and closest feature types between a convex and a triangle
	template<typename ConvexType>
	bool FindClosestFeatures(const ConvexType& Convex, const FTriangle& Triangle, const FVec3& TriangleNormal, const FVec3& ConvexRelativeMovement, const FReal CullDistance, FConvexContactPoint& OutContactPoint)
	{
		// Find the closest point on the convex and triangle that we will use to generate the manifold
		// NOTE: use an upper limit on cull distance here since the real cull distance depends on the motion against the contact normal which we don't know yet
		const FReal EarlyCullDistanceSq = TNumericLimits<FReal>::Max();	// @todo(chaos): put something useful here
		if (!Private::ConvexTriangleContactPoint(Convex, Triangle, TriangleNormal, EarlyCullDistanceSq, OutContactPoint))
		{
			return false;
		}

		// Now check the cull distance, taking movement into account
		const FReal SeparationWithMotion = OutContactPoint.Phi + FVec3::DotProduct(ConvexRelativeMovement, OutContactPoint.ShapeContactNormal);
		if ((OutContactPoint.Phi > CullDistance) && (SeparationWithMotion > CullDistance))
		{
			return false;
		}

		// Find the triangle feature at the contact point
		if (!Private::GetTriangleFeature(Triangle, TriangleNormal, OutContactPoint.ShapeContactPoints[1], OutContactPoint.ShapeContactNormal, OutContactPoint.Features[1]))
		{
			return false;
		}

		// Find the convex feature at the contact point
		if (!Private::GetConvexFeature(Convex, OutContactPoint.ShapeContactPoints[0], -OutContactPoint.ShapeContactNormal, OutContactPoint.Features[0]))
		{
			return false;
		}

		return true;
	}

	template<>
	bool FindClosestFeatures(const FImplicitSphere3& Convex, const FTriangle& Triangle, const FVec3& TriangleNormal, const FVec3& ConvexRelativeMovement, const FReal CullDistance, FConvexContactPoint& OutContactPoint)
	{
		const FVec3 Center = Convex.GetCenterf();
		FVec3 Bary = FVec3(0);

		const FVec3 NearestPointTriangle = FindClosestPointAndBaryOnTriangle(Triangle.GetVertex(0), Triangle.GetVertex(1), Triangle.GetVertex(2), Center, Bary);
		const FReal Distance = FVec3::Dist(NearestPointTriangle, Center) - Convex.GetRadiusf();

		OutContactPoint.ShapeContactPoints[0] = Convex.FindClosestPoint(NearestPointTriangle);
		OutContactPoint.ShapeContactPoints[1] = NearestPointTriangle;
		OutContactPoint.ShapeContactNormal = TriangleNormal;
		OutContactPoint.ShapeContactNormal.Normalize();
		OutContactPoint.Phi = Distance;

		// Now check the cull distance, taking movement into account
		const FReal SeparationWithMotion = OutContactPoint.Phi + FVec3::DotProduct(ConvexRelativeMovement, OutContactPoint.ShapeContactNormal);
		if ((OutContactPoint.Phi > CullDistance) && (SeparationWithMotion > CullDistance))
		{
			return false;
		}

		// Find the triangle feature at the contact point
		if (!Private::GetTriangleFeature(Triangle, TriangleNormal, OutContactPoint.ShapeContactPoints[1], OutContactPoint.ShapeContactNormal, OutContactPoint.Features[1]))
		{
			return false;
		}

		// Find the convex feature at the contact point
		OutContactPoint.Features[0].FeatureType = EConvexFeatureType::Vertex;
		OutContactPoint.Features[0].PlaneIndex = 0;
		OutContactPoint.Features[0].PlaneFeatureIndex = 0;

		return true;
	}

	template bool FindClosestFeatures(
		const FImplicitConvex3& Convex,
		const FTriangle& Triangle, 
		const FVec3& TriangleNormal,
		const FVec3& ConvexRelativeMovement,
		const FReal CullDistance, 
		FConvexContactPoint& OutContact);

	template bool FindClosestFeatures(
		const TImplicitObjectInstanced<FImplicitConvex3>& Convex,
		const FTriangle& Triangle,
		const FVec3& TriangleNormal,
		const FVec3& ConvexRelativeMovement,
		const FReal CullDistance,
		FConvexContactPoint& OutContact);

	template bool FindClosestFeatures(
		const TImplicitObjectScaled<FImplicitConvex3>& Convex,
		const FTriangle& Triangle,
		const FVec3& TriangleNormal,
		const FVec3& ConvexRelativeMovement,
		const FReal CullDistance,
		FConvexContactPoint& OutContact);

	template bool FindClosestFeatures(
		const FImplicitBox3& Convex,
		const FTriangle& Triangle,
		const FVec3& TriangleNormal,
		const FVec3& ConvexRelativeMovement,
		const FReal CullDistance,
		FConvexContactPoint& OutContact);

	template bool FindClosestFeatures(
		const TImplicitObjectScaled<FImplicitBox3>& Convex,
		const FTriangle& Triangle,
		const FVec3& TriangleNormal,
		const FVec3& ConvexRelativeMovement,
		const FReal CullDistance,
		FConvexContactPoint& OutContact);

	template bool FindClosestFeatures(
		const TImplicitObjectInstanced<FImplicitBox3>& Convex,
		const FTriangle& Triangle,
		const FVec3& TriangleNormal,
		const FVec3& ConvexRelativeMovement,
		const FReal CullDistance,
		FConvexContactPoint& OutContact);

} // Chaos::Private
