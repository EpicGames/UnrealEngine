// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Chaos/CastingUtilities.h"
#include "Chaos/Core.h"
#include "Chaos/ImplicitFwd.h"
#include "Chaos/Collision/ContactPoint.h"
#include "Chaos/GJK.h"


namespace Chaos
{
	template <typename GeometryA, typename GeometryB>
	FContactPoint GJKContactPoint2(const GeometryA& A, const GeometryB& B, const FRigidTransform3& ATM, const FRigidTransform3& BToATM, const FVec3& InitialDir)
	{
		FContactPoint Contact;

		FReal Penetration;
		FVec3 ClosestA, ClosestBInA, Normal;
		int32 ClosestVertexIndexA, ClosestVertexIndexB;

		// Slightly increased epsilon to reduce error in normal for almost touching objects.
		const FReal Epsilon = 3.e-3f;

		if (GJKPenetration<true>(A, B, BToATM, Penetration, ClosestA, ClosestBInA, Normal, ClosestVertexIndexA, ClosestVertexIndexB, FReal(0), FReal(0), InitialDir, Epsilon))
		{
			// GJK output is all in the local space of A. We need to transform the B-relative position and the normal in to B-space
			Contact.ShapeContactPoints[0] = ClosestA;
			Contact.ShapeContactPoints[1] = BToATM.InverseTransformPosition(ClosestBInA);
			Contact.ShapeContactNormal = -BToATM.InverseTransformVector(Normal);
			Contact.Phi = -Penetration;
		}

		return Contact;
	}

	template <typename GeometryA, typename GeometryB>
	FContactPoint GJKContactPoint(const GeometryA& A, const FRigidTransform3& ATM, const GeometryB& B, const FRigidTransform3& BTM, const FVec3& InitialDir)
	{
		const FRigidTransform3 BToATM = BTM.GetRelativeTransform(ATM);
		return GJKContactPoint2(A, B, ATM, BToATM, InitialDir);
	}
}