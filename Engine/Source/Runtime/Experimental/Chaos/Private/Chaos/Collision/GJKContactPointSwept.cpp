// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/Collision/GJKContactPointSwept.h"

#include "Chaos/CastingUtilities.h"
#include "Chaos/Collision/ContactPoint.h"
#include "Chaos/Collision/ContactPointsMiscShapes.h"
#include "Chaos/GJK.h"

namespace Chaos
{
	FContactPoint GJKContactPointSwept(const FGeomGJKHelperSIMD& A, const FRigidTransform3& AStartTM, const FRigidTransform3& AEndTM, const FGeomGJKHelperSIMD& B, const FRigidTransform3& BStartTM, const FRigidTransform3& BEndTM, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& OutTOI)
	{
		FContactPoint Contact;
		/** B could be static or dynamic. In both cases, we compute contact point in B local space.  
		* GJKRaycast2 assumes that A, B do not rotate. 
		* If B is rotating, the trajectory of A in space B will become non-linear. Also if A or B is rotating, the minkowski difference of the sweep geometry will be very complex and the sweep geometry can be even non-convex, creating difficulties for the GJK algorithm.
		* For computational efficiency, we could pick any rotation from the start frame rotation to the end frame rotation and lock that rotation for the sweeping process. No matter what rotation we pick, we could always get problems when rotation is large.
		* If we pick the start frame rotation, we can think of this whole process as first sweeping the objects from X to P, then rotating them in place to end frame rotations. We might end up with A and B penetrating because of the end frame rotations. So here instead, we use the end frame rotations. This process becomes first rotating A and B in place to end frame rotations, then sweeping them from X to P. This produces less penetrations at the end of the frame.
		*/
		const FRigidTransform3 ATM(AStartTM.GetLocation(), AEndTM.GetRotation());
		const FRigidTransform3 BTM(BStartTM.GetLocation(), BEndTM.GetRotation());
		const FRigidTransform3 AToBTM = ATM.GetRelativeTransform(BTM);
		const FVec3 LocalDir = BStartTM.InverseTransformVectorNoScale(Dir); 

		FReal Distance;
		FVec3 Location, Normal;
		if (GJKRaycast2(B, A, AToBTM, LocalDir, Length, Distance, Location, Normal, (FReal)0, true))
		{
			const FReal DirDotNormal = FVec3::DotProduct(LocalDir, Normal);
			FReal TargetTOI, TargetPhi;
			if (ComputeSweptContactTOIAndPhiAtTargetPenetration(DirDotNormal, Length, Distance, IgnorePenetration, TargetPenetration, TargetTOI, TargetPhi))
			{
				// GJK output is all in the local space of B. We need to transform the B-relative position and the normal in to B-space
				FRigidTransform3 ATOITM, BTOITM;
				if (Distance > 0.f)
				{
					ATOITM = FRigidTransform3(AStartTM.GetLocation() * (1 - TargetTOI) + AEndTM.GetLocation() * TargetTOI, AEndTM.GetRotation());
					BTOITM = FRigidTransform3(BStartTM.GetLocation() * (1 - TargetTOI) + BEndTM.GetLocation() * TargetTOI, BEndTM.GetRotation());
				}
				else
				{
					ATOITM = ATM;
					BTOITM = BTM;
				}

				const FVec3 WorldLocation = BTOITM.TransformPosition(Location);
				Contact.ShapeContactPoints[0] = ATOITM.InverseTransformPosition(WorldLocation);
				Contact.ShapeContactPoints[1] = Location;
				Contact.ShapeContactNormal = Normal;
				Contact.Phi = TargetPhi;
				OutTOI = TargetTOI;
			}
		}

		return Contact;
	}

	FContactPoint GenericConvexConvexContactPointSwept(const FImplicitObject& A, const FRigidTransform3& AStartTM, const FRigidTransform3& AEndTM, const FImplicitObject& B, const FRigidTransform3& BStartTM, const FRigidTransform3& BEndTM, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& TOI)
	{
		// This expands to a switch of switches that calls the inner function with the appropriate concrete implicit types
		return Utilities::CastHelperNoUnwrap(A, AStartTM, AEndTM, [&](const auto& ADowncast, const FRigidTransform3& AFullStartTM, const FRigidTransform3& AFullEndTM)
		{
			return Utilities::CastHelperNoUnwrap(B, BStartTM, BEndTM, [&](const auto& BDowncast, const FRigidTransform3& BFullStartTM, const FRigidTransform3& BFullEndTM)
			{
				return GJKContactPointSwept(ADowncast, AFullStartTM, AFullEndTM, BDowncast, BFullStartTM, BFullEndTM, Dir, Length, IgnorePenetration, TargetPenetration, TOI);
			});
		});
	}
}