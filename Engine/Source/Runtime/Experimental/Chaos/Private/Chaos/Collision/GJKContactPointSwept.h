// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/CollisionResolutionTypes.h"
#include "Chaos/ImplicitFwd.h"

namespace Chaos
{
	template<typename T> class TContactPoint;
	using FContactPoint = TContactPoint<FReal>;

	FContactPoint GenericConvexConvexContactPointSwept(const FImplicitObject& A, const FRigidTransform3& AStartTM, const FRigidTransform3& AEndTM, const FImplicitObject& B, const FRigidTransform3& BStartTM, const FRigidTransform3& BEndTM, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& TOI);
}
