// Copyright Epic Games, Inc.All Rights Reserved.
#include "Chaos/Collision/MeshContactGenerator.h"
#include "Chaos/Collision/ConvexFeature.h"
#include "Chaos/DebugDrawQueue.h"

namespace Chaos
{
	extern FRealSingle Chaos_Collision_MeshContactNormalThreshold;
	extern FRealSingle Chaos_Collision_MeshContactNormalRejectionThreshold;
	extern int32 Chaos_Collision_MeshManifoldHashSize;
}

namespace Chaos::CVars
{
	extern int32 ChaosSolverDebugDrawMeshContacts;

	bool bMeshContactGeneratorFixContactNormalFixEnabled = true;
	FAutoConsoleVariableRef CVarChaosMeshContactGeneratorFixContactNormalFixEnabled(TEXT("p.Chaos.MeshContactGenerator.FixContactNormal.FixEnabled"), bMeshContactGeneratorFixContactNormalFixEnabled, TEXT("Until new code path is well tested"));

	bool bMeshContactGeneratorTreatBoundaryVertexAsFace = true;
	FAutoConsoleVariableRef CVarChaosMeshContactGeneratorTreatBoundaryVertexAsFace(TEXT("p.Chaos.MeshContactGenerator.TreatBoundaryVertexAsFace"), bMeshContactGeneratorTreatBoundaryVertexAsFace, TEXT("Treat boundary vertex collisions as face collisions"));

	bool bMeshContactGeneratorTreatBoundaryEdgeAsFace = true;
	FAutoConsoleVariableRef CVarChaosMeshContactGeneratorTreatBoundaryEdgeAsFace(TEXT("p.Chaos.MeshContactGenerator.TreatBoundaryEdgeAsFace"), bMeshContactGeneratorTreatBoundaryEdgeAsFace, TEXT("Treat boundary edge collisions as face collisions"));
}

namespace Chaos::Private
{
	FMeshContactGeneratorSettings::FMeshContactGeneratorSettings()
	{
		HashSize = FMath::RoundUpToPowerOfTwo(Chaos_Collision_MeshManifoldHashSize);
		FaceNormalDotThreshold = Chaos_Collision_MeshContactNormalThreshold;
		EdgeNormalDotRejectTolerance = Chaos_Collision_MeshContactNormalRejectionThreshold;
		BackFaceCullTolerance = FReal(1.e-6);
		BarycentricTolerance = FReal(1.e-3);
		MaxContactsBufferSize = 1000;
		bCullBackFaces = true;
		bFixNormals = true;
		bSortByPhi = false;
		bSortForSolverConvergence = false;
		bUseTwoPassLoop = true;
	}

	FMeshContactGenerator::FMeshContactGenerator(const FMeshContactGeneratorSettings& InSettings)
		: Settings(InSettings)
		, EdgeTriangleIndicesMap(InSettings.HashSize)
		, VertexContactDataMap(InSettings.HashSize)
		, EdgeContactDataMap(InSettings.HashSize)
	{
	}

	void FMeshContactGenerator::Reset(const int32 InMaxTriangles, const int32 InMaxContacts)
	{
		Triangles.Reset(InMaxTriangles);

		Contacts.Reset(InMaxContacts);
		ContactDatas.Reset(InMaxContacts);

		// If all edges were shared between 2 triangles we have NumEdges = (3 * NumTriangles) / 2
		// but not all are shared so lets go with NumEdges = (2 * NumTriangles)
		EdgeTriangleIndicesMap.Reset(InMaxTriangles * 2);

		// Worst case - assume all contacts are vertex contacts or edge contacts
		VertexContactDataMap.Reset(InMaxContacts);
		EdgeContactDataMap.Reset(InMaxContacts);
	}

	void FMeshContactGenerator::AddTriangleContacts(const int32 LocalTriangleIndex, const TArrayView<FContactPoint>& TriangleContactPoints)
	{
		FTriangleExt& Triangle = Triangles[LocalTriangleIndex];

		// We need to know how many vertex and edge collisions a triangle has on it (from collision with other triangle 
		// that share edges and vertices) so we avoid visiting triangles if we already know the contacts. Check all
		// contacts and assigne edge/vertex status as required. See GenerateMeshContacts()
		for (const FContactPoint& ContactPoint : TriangleContactPoints)
		{
			// Clamp the number of contacts we can add per mesh
			if (Contacts.Num() == Contacts.Max())
			{
				break;
			}

			// See if we have a vertex or edge contact
			// @todo(chaos): we should be able to produce this as output from the manifold creator
			int32 LocalVertexID0, LocalVertexID1;
			GetTriangleEdgeVerticesAtPosition(
				ContactPoint.ShapeContactPoints[1],
				Triangle.GetVertex(0), Triangle.GetVertex(1), Triangle.GetVertex(2),
				LocalVertexID0, LocalVertexID1,
				Settings.BarycentricTolerance);

			const int32 VertexID0 = (LocalVertexID0 != INDEX_NONE) ? Triangle.GetVertexIndex(LocalVertexID0) : INDEX_NONE;
			const int32 VertexID1 = (LocalVertexID1 != INDEX_NONE) ? Triangle.GetVertexIndex(LocalVertexID1) : INDEX_NONE;

			const int32 NewContactIndex = Contacts.Num();

			const FRealSingle ContactNormalDotTriangleNormal = FRealSingle(FVec3::DotProduct(ContactPoint.ShapeContactNormal, Triangle.GetNormal()));

			const bool bIsFaceContact = (ContactNormalDotTriangleNormal > Settings.FaceNormalDotThreshold);

			// Register vertex and edge collisions
			if ((VertexID0 != INDEX_NONE) && (VertexID1 != INDEX_NONE))
			{
				// Edge collisions - keep the contacts from only one of the triangles sharing the edge.
				// NOTES:
				//  - for convex-vs-triangle collisions we can have up to 2 collisions per triangle edge.
				//	- each triangle sharing the edge should report the same 0, 1, or 2 collisions which
				//	  means we can ignore contacts from one of the triangles.
				//	- except that convex-triangle manifold creation limits contacts to 4 so it may have 
				//	  thrown away different edge contacts from each triangle. This should be fine though
				//	  since it only happens when we already have a lot of contacts on the triangles so
				//	  we don't need those edge contacts.
				FContactEdgeID EdgeID = FContactEdgeID(VertexID0, VertexID1);

				// If we already have collisions with this edge on another triangle,
				// keep those with the normal closest to its face normal
				FEdgeContactData* EdgeContact = EdgeContactDataMap.Find(EdgeID);
				if (EdgeContact != nullptr)
				{
					// We have an existing contact on this edge. Is it on another triangle?
					if (EdgeContact->LocalTriangleIndex != LocalTriangleIndex)
					{
						// If existing contact was a face contact just keep it and ignore the new contact.
						if (EdgeContact->bIsFaceContact)
						{
							continue;
						}
							
						// If the existing contact had a normal closer to its face normal, keep it and ignore the new contact
						if (!bIsFaceContact && (ContactNormalDotTriangleNormal < EdgeContact->ContactNormalDotTriangleNormal))
						{
							continue;
						}

						// Throw away the existing edge contacts
						if (EdgeContact->ContactIndices[0] != INDEX_NONE)
						{
							ContactDatas[EdgeContact->ContactIndices[0]].SetDisabled();
						}
						if (EdgeContact->ContactIndices[1] != INDEX_NONE)
						{
							ContactDatas[EdgeContact->ContactIndices[1]].SetDisabled();
						}

						// Change existing edge contact to refer to this triangle
						EdgeContact->Init(EdgeID, LocalTriangleIndex);
					}
				}
				else
				{
					// First time we have seen this edge - start tracking its contacts
					EdgeContact = &EdgeContactDataMap.Emplace(EdgeID, EdgeID, LocalTriangleIndex);
				}

				// Track this edge contact
				check(EdgeContact != nullptr);
				EdgeContact->AddContact(NewContactIndex, ContactNormalDotTriangleNormal, bIsFaceContact);

				// If it's a face contact, tell each triangle using the edge it has a face contact. 
				// This is used in the outer loop to skip triangles that have multiple contacts from 
				// adjacent triangle edges or verts.
				if (bIsFaceContact)
				{
					Triangles[LocalTriangleIndex].AddFaceEdgeCollision();

					const int32 OtherLocalTriangleIndex = GetOtherTriangleIndexForEdge(LocalTriangleIndex, EdgeID);
					if (OtherLocalTriangleIndex != INDEX_NONE)
					{
						Triangles[OtherLocalTriangleIndex].AddFaceEdgeCollision();
					}
				}
			}
			else if ((VertexID0 != INDEX_NONE) && (VertexID1 == INDEX_NONE))
			{
				// Vertex collisions - we only keep one vertex collision per vertex
				if (FVertexContactData* ExistingVertexContact = VertexContactDataMap.Find(VertexID0))
				{
					// We have an existing contact at this vertex. 
					// If it was a face contact just ignore the new contact, 
					// otherwise keep the contact with the most face-pointing normal.
					if (!ExistingVertexContact->bIsFaceContact)
					{
						FContactPoint& ExistingContact = Contacts[ExistingVertexContact->ContactIndex];
						FTriangleContactPointData& ExistingContactData = ContactDatas[ExistingVertexContact->ContactIndex];
						if (ContactNormalDotTriangleNormal > ExistingContactData.GetContactNormalDotTriangleNormal())
						{
							ExistingContact = ContactPoint;
							ExistingContact.FaceIndex = Triangle.GetTriangleIndex();

							ExistingContactData.SetVertexID(VertexID0);
							ExistingContactData.SetContactNormalDotTriangleNormal(ContactNormalDotTriangleNormal);
							ExistingContactData.SetTriangleIndex(LocalTriangleIndex);

							ExistingVertexContact->bIsFaceContact = bIsFaceContact;
						}
					}

					// Don't add this contact
					continue;
				}

				// This is the first time we hit this vertex so store the contact index with the vertex
				VertexContactDataMap.Emplace(VertexID0, VertexID0, NewContactIndex, bIsFaceContact);
			}

			// Store the contact, set the face index
			Contacts.Add(ContactPoint);
			Contacts[NewContactIndex].FaceIndex = Triangle.GetTriangleIndex();

			// Set the contact metadata and enable it
			ContactDatas.AddDefaulted();
			ContactDatas[NewContactIndex].SetEdgeOrVertexID({ VertexID0, VertexID1 });
			ContactDatas[NewContactIndex].SetContactNormalDotTriangleNormal(ContactNormalDotTriangleNormal);
			ContactDatas[NewContactIndex].SetTriangleIndex(LocalTriangleIndex);
			ContactDatas[NewContactIndex].SetEnabled();
		}
	}

	void FMeshContactGenerator::ProcessGeneratedContacts(const FRigidTransform3& ConvexTransform, const FRigidTransform3& MeshToConvexTransform)
	{
		if (Contacts.Num() == 0)
		{
			return;
		}

		if (!!Settings.bFixNormals)
		{
			// Contacts that get pruned or corrected will show as green
			DebugDrawContacts(ConvexTransform, FColor::Green, 0.5);
		}

		PruneAndCorrectContacts();

		// Final contacts will be red
		DebugDrawContacts(ConvexTransform, FColor::Red, 1.0);

		// Visited triangles are white, ignored triangles are gray
		DebugDrawTriangles(ConvexTransform, FColor::White, FColor::Silver);

		if (!!Settings.bSortByPhi)
		{
			SortContactByPhi();
		}

		if (!!Settings.bSortForSolverConvergence)
		{
			SortContactsForSolverConvergence();
		}

		FinalizeContacts(MeshToConvexTransform);
	}

	void FMeshContactGenerator::PruneAndCorrectContacts()
	{
		for (int32 ContactIndex = 0; ContactIndex < Contacts.Num(); ++ContactIndex)
		{
			FTriangleContactPointData& ContactPointData = ContactDatas[ContactIndex];

			// Some contacts may be disabled during the AddContacts phase
			if (!ContactPointData.IsEnabled())
			{
				continue;
			}

			// Reject back-faces
			if (!!Settings.bCullBackFaces && (ContactPointData.GetContactNormalDotTriangleNormal() < -Settings.BackFaceCullTolerance))
			{
				ContactPointData.SetDisabled();
				continue;
			}

			// Fix edge normals that aren't already close to a face normal
			if (!!Settings.bFixNormals && (ContactPointData.GetContactNormalDotTriangleNormal() < Settings.FaceNormalDotThreshold))
			{
				FixContactNormal(ContactIndex);
			}
		}

		// re-pack the contact array
		RemoveDisabledContacts();
	}

	void FMeshContactGenerator::FixContactNormal(const int32 ContactIndex)
	{
		FContactPoint& ContactPoint = Contacts[ContactIndex];
		FTriangleContactPointData& ContactPointData = ContactDatas[ContactIndex];

		const int32 LocalTriangleIndex = ContactPointData.GetTriangleIndex();
		const FTriangleExt& Triangle = Triangles[LocalTriangleIndex];
		const FVec3& TriangleNormal = Triangle.GetNormal();

		const FReal ContactDotNormal = ContactPointData.GetContactNormalDotTriangleNormal();

		// If we have an edge or vertex contact, make sure that the normal is in a valiid range, based
		// on the triangles that share that edge or vertex.
		if (ContactPointData.IsEdge())
		{
			const int32 OtherLocalTriangleIndex = GetOtherTriangleIndexForEdge(LocalTriangleIndex, ContactPointData.GetEdgeID());
			if (OtherLocalTriangleIndex != INDEX_NONE)
			{
				// This is a shared edge, so the contact normal must lie between the normals of the two faces using the edge.
				const FTriangleExt& OtherTriangle = Triangles[OtherLocalTriangleIndex];
				const FVec3& OtherTriangleNormal = OtherTriangle.GetNormal();

				const FReal MinContactDotNormal = FRealSingle(FVec3::DotProduct(OtherTriangleNormal, TriangleNormal));
				if (ContactDotNormal < MinContactDotNormal)
				{
					// We are outside the valid normal range for this edge
					// If our normal was very far away from a valid normal, drop the contact
					if (MinContactDotNormal - ContactDotNormal > Settings.EdgeNormalDotRejectTolerance)
					{
						ContactPointData.SetDisabled();
						return;
					}

					// Convert the edge collision to a face collision on one of the faces, selected to get the smallest depth
					FVec3 CorrectedContactNormal;
					int32 CorrectedTriangleIndex;
					const FReal OtherContactDotNormal = FVec3::DotProduct(ContactPoint.ShapeContactNormal, OtherTriangleNormal);
					if (ContactDotNormal >= OtherContactDotNormal)
					{
						CorrectedContactNormal = (ContactDotNormal > -SMALL_NUMBER) ? TriangleNormal : -TriangleNormal;
						CorrectedTriangleIndex = LocalTriangleIndex;
					}
					else
					{
						CorrectedContactNormal = (OtherContactDotNormal > -SMALL_NUMBER) ? OtherTriangleNormal : -OtherTriangleNormal;
						CorrectedTriangleIndex = OtherLocalTriangleIndex;
					}

					// NOTE: We keep the contact depth as it is because we know that the depth is a lower-bound. 
					// We have to update the ShapeContactPoint[0] because Phi is actually derived from the positions (the value in ContactPoint is just a cache of current state)
					// @todo(chaos): face selection logic might be better if we knew the contact velocity
					ContactPoint.ShapeContactNormal = CorrectedContactNormal;
					ContactPoint.ShapeContactPoints[0] = ContactPoint.ShapeContactPoints[1] + ContactPoint.Phi * CorrectedContactNormal;
					ContactPointData.SetTriangleIndex(CorrectedTriangleIndex);
				}
			}
			else if (CVars::bMeshContactGeneratorTreatBoundaryEdgeAsFace)
			{
				// This is a boundary edge, so remap the contact to a face contact.
				// @todo(chaos): ideally we would have information about edge sharing between adjancent meshes, in which case
				// we could perform normal correction in all cases.
				ContactPoint.ShapeContactNormal = TriangleNormal;
				ContactPoint.ShapeContactPoints[0] = ContactPoint.ShapeContactPoints[1] + ContactPoint.Phi * TriangleNormal;
			}
		}
		else if (ContactPointData.IsVertex())
		{
			// We have a vertex collision. Ensure that the contact normal is in a valid range for the vertex based on the
			// triangles that share the vertex (there can be arbitarily many of these).

			// NOTE: the code in the cvar fixes an issue which would leave invalid normals on spikey meshes,
			// but we're very close to a release, so adding option to rollback if this version is flased
			if (CVars::bMeshContactGeneratorFixContactNormalFixEnabled)
			{
				const int32 VertexIndexA = ContactPointData.GetVertexID();
				FVec3 VertexA;
				if (!Triangle.GetVertexWithID(VertexIndexA, VertexA))
				{
					// @todo(chaos): this is an error condition
					return;
				}

				// @todo(chaos): the map of Vertex->TriangleIndices would help here but may not be a net win
				for (int32 OtherLocalTriangleIndex = 0; OtherLocalTriangleIndex < Triangles.Num(); ++OtherLocalTriangleIndex)
				{
					const FTriangleExt& OtherTriangle = Triangles[OtherLocalTriangleIndex];

					// If we have a boundary vertex, remap the contact to a face context. The vertex is a boundary 
					// vertex if any of the edges including the vertex are boundary edges (i.e., not shared between two triangles)
					if (CVars::bMeshContactGeneratorTreatBoundaryVertexAsFace)
					{
						int32 OtherVertexIndexB, OtherVertexIndexC;
						if (OtherTriangle.GetOtherVertexIDs(VertexIndexA, OtherVertexIndexB, OtherVertexIndexC))
						{
							const FContactEdgeID EdgeAB = FContactEdgeID(VertexIndexA, OtherVertexIndexB);
							const FContactEdgeID EdgeCA = FContactEdgeID(OtherVertexIndexC, VertexIndexA);
							if (!IsSharedEdge(EdgeAB) || !IsSharedEdge(EdgeCA))
							{
								ContactPoint.ShapeContactNormal = TriangleNormal;
								ContactPoint.ShapeContactPoints[0] = ContactPoint.ShapeContactPoints[1] + ContactPoint.Phi * TriangleNormal;
								break;
							}
						}
					}

					FVec3 OtherVertexB, OtherVertexC;
					if (OtherTriangle.GetOtherVerticesFromID(VertexIndexA, OtherVertexB, OtherVertexC))
					{
						// Does the contact normal point into the infinite prism formed by extruding the triangle along the face normal?
						const FVec3& OtherTriangleNormal = OtherTriangle.GetNormal();
						const FVec3 OtherEdge0 = OtherVertexB - VertexA;
						const FVec3 OtherEdge1 = VertexA - OtherVertexC;
						const FVec3 OtherEdgeNormal0 = FVec3::CrossProduct(OtherTriangleNormal, OtherEdge0);	// Not normlized
						const FVec3 OtherEdgeNormal1 = FVec3::CrossProduct(OtherTriangleNormal, OtherEdge1);	// Not normlized
						const FReal OtherEdgeSign0 = FVec3::DotProduct(ContactPoint.ShapeContactNormal, OtherEdgeNormal0);
						const FReal OtherEdgeSign1 = FVec3::DotProduct(ContactPoint.ShapeContactNormal, OtherEdgeNormal1);
						const FReal OtherEdgeSign0Sq = OtherEdgeSign0 * FMath::Abs(OtherEdgeSign0);
						const FReal OtherEdgeSign1Sq = OtherEdgeSign1 * FMath::Abs(OtherEdgeSign1);
						const FReal NormalToleranceSq = FReal(1.e-8);
						const FReal NormalTolerance0Sq = NormalToleranceSq * OtherEdge0.SizeSquared();
						const FReal NormalTolerance1Sq = NormalToleranceSq * OtherEdge1.SizeSquared();
						if ((OtherEdgeSign0Sq >= FReal(-NormalTolerance0Sq)) && (OtherEdgeSign1Sq >= FReal(-NormalTolerance1Sq)))
						{
							ContactPoint.ShapeContactNormal = OtherTriangleNormal;
							ContactPoint.ShapeContactPoints[0] = ContactPoint.ShapeContactPoints[1] + ContactPoint.Phi * OtherTriangleNormal;
							ContactPointData.SetTriangleIndex(OtherLocalTriangleIndex);
							break;
						}
					}
				}
			}
			else
			{
				//
				//
				// @todo(chaos): remove this branch when above code is well tested
				//
				//
				for (int32 TriangleIndex = 0; TriangleIndex < Triangles.Num(); ++TriangleIndex)
				{
					const FTriangleExt& OtherTriangle = Triangles[TriangleIndex];
					if ((TriangleIndex != LocalTriangleIndex) && OtherTriangle.HasVertexID(ContactPointData.GetVertexID()))
					{
						FVec3 VertexA, VertexB, VertexC;
						if (OtherTriangle.GetVertexWithID(ContactPointData.GetVertexID(), VertexA) && OtherTriangle.GetOtherVerticesFromID(ContactPointData.GetVertexID(), VertexB, VertexC))
						{
							// Does the contact normal point into the infinite prism formed by extruding the triangle along the face normal?
							// It does if the contact normal dotted with the edge plane normal is negative for both edge planes on the triangle that use the vertex.
							const FVec3 EdgeDelta0 = VertexB - VertexA;
							const FVec3 EdgeDelta1 = VertexC - VertexA;
							const FVec3& OtherTriangleNormal = OtherTriangle.GetNormal();

							const FReal EdgeSign0 = FVec3::DotProduct(FVec3::CrossProduct(ContactPoint.ShapeContactNormal, VertexB - VertexA), OtherTriangleNormal);
							const FReal EdgeSign1 = FVec3::DotProduct(FVec3::CrossProduct(ContactPoint.ShapeContactNormal, VertexC - VertexA), OtherTriangleNormal);
							if (FMath::Sign(EdgeSign0) == FMath::Sign(EdgeSign1))
							{
								const FVec3 Centroid = OtherTriangle.GetCentroid();
								const FReal NormalDotCentroid = FVec3::DotProduct(ContactPoint.ShapeContactNormal, Centroid - ContactPoint.ShapeContactPoints[1]);
								if (NormalDotCentroid > 0)
								{
									// If our normal was very far away from a valid normal, drop the contact
									const FReal MinContactDotNormal = FRealSingle(FVec3::DotProduct(OtherTriangleNormal, TriangleNormal));
									if (MinContactDotNormal - ContactDotNormal > Settings.EdgeNormalDotRejectTolerance)
									{
										ContactPointData.SetDisabled();
										return;
									}

									const FReal OtherContactDotNormal = FVec3::DotProduct(ContactPoint.ShapeContactNormal, OtherTriangleNormal);
									const FVec3 CorrectedContactNormal = OtherTriangleNormal;

									ContactPoint.ShapeContactNormal = CorrectedContactNormal;
									ContactPoint.ShapeContactPoints[0] = ContactPoint.ShapeContactPoints[1] + ContactPoint.Phi * CorrectedContactNormal;
								}
							}
						}
					}
				}
			}
		}
	}

	bool FMeshContactGenerator::FixFeature(const int32 LocalTriangleIndex, Private::EConvexFeatureType& InOutFeatureType, int32& InOutFeatureIndex, FVec3& InOutPlaneNormal)
	{
		const FTriangleExt& Triangle = Triangles[LocalTriangleIndex];
		const FVec3& TriangleNormal = Triangle.GetNormal();

		// For convex edges, we ensure that the normal is between the normals of the adjacent faces
		// For concave edges, we replace the normal with the triangle face normal
		if (InOutFeatureType == Private::EConvexFeatureType::Edge)
		{
			check(InOutFeatureIndex != INDEX_NONE);

			const int32 LocalVertexIndex0 = InOutFeatureIndex;
			const int32 LocalVertexIndex1 = (InOutFeatureIndex == 2) ? 0 : InOutFeatureIndex + 1;
			const int32 VertexIndex0 = Triangle.GetVertexIndex(LocalVertexIndex0);
			const int32 VertexIndex1 = Triangle.GetVertexIndex(LocalVertexIndex1);
			const FContactEdgeID EdgeID = FContactEdgeID(VertexIndex0, VertexIndex1);

			const int32 OtherLocalTriangleIndex = GetOtherTriangleIndexForEdge(LocalTriangleIndex, EdgeID);

			// If there is no other triangle we have a boundary edge
			if (OtherLocalTriangleIndex == INDEX_NONE)
			{
				// Boundary edge - use face normal
				InOutFeatureType = Private::EConvexFeatureType::Plane;
				InOutFeatureIndex = 0;
				InOutPlaneNormal = TriangleNormal;
				return true;
			}

			const FTriangleExt& OtherTriangle = Triangles[OtherLocalTriangleIndex];
			const FVec3& OtherTriangleNormal = OtherTriangle.GetNormal();

			// Common case - both triangles have the same normal - treat as concave
			const FReal NormalEpsilon = FReal(1.e-6);
			const FReal TriangleNormalsDot = FVec3::DotProduct(TriangleNormal, OtherTriangleNormal);
			if (FMath::IsNearlyEqual(TriangleNormalsDot, FReal(1), NormalEpsilon))
			{
				// Concave edge - use the face normal
				InOutFeatureType = Private::EConvexFeatureType::Plane;
				InOutFeatureIndex = 0;
				InOutPlaneNormal = TriangleNormal;
				return true;
			}

			// If normals are different do a full concavity check
			const FVec3 EdgeDelta = Triangle.GetVertex(LocalVertexIndex1) - Triangle.GetVertex(LocalVertexIndex0);
			const FVec3 TriangleEdgeNormalVector = FVec3::CrossProduct(EdgeDelta, TriangleNormal);	// Not normalized
			const FReal OtherFaceNormalDotEdgeNormal = FVec3::DotProduct(OtherTriangleNormal, TriangleEdgeNormalVector);
			if (OtherFaceNormalDotEdgeNormal < FReal(0))
			{
				// Concave edge - use the face normal
				InOutFeatureType = Private::EConvexFeatureType::Plane;
				InOutFeatureIndex = 0;
				InOutPlaneNormal = TriangleNormal;
				return true;
			}

			// We have a convex edge. Ensure the normal is in the valid region
			const FReal NormalDotEdge = FVec3::DotProduct(InOutPlaneNormal, TriangleEdgeNormalVector);
			if (NormalDotEdge < FReal(0))
			{
				InOutFeatureType = Private::EConvexFeatureType::Plane;
				InOutFeatureIndex = 0;
				InOutPlaneNormal = TriangleNormal;
				return true;
			}

			// Same as above but against the other triangle sharing the edge
			FVec3 OtherVertex0, OtherVertex1;
			if (OtherTriangle.GetVertexWithID(VertexIndex0, OtherVertex0) && OtherTriangle.GetVertexWithID(VertexIndex1, OtherVertex1))
			{
				const FVec3 OtherEdgeDelta = OtherVertex0 - OtherVertex1;
				const FVec3 OtherTriangleEdgeNormalVector = FVec3::CrossProduct(OtherEdgeDelta, OtherTriangleNormal);	// Not normalized
				const FReal OtherNormalDotEdge = FVec3::DotProduct(InOutPlaneNormal, OtherTriangleEdgeNormalVector);
				if (OtherNormalDotEdge < FReal(0))
				{
					InOutFeatureType = Private::EConvexFeatureType::Plane;
					InOutFeatureIndex = 0;
					InOutPlaneNormal = OtherTriangleNormal;
					return true;
				}
			}
		}

		// For vertices, ensure that the contact normal is in a valid range for the vertex based on the
		// triangles that share the vertex (there can be arbitarily many of these).
		if (InOutFeatureType == Private::EConvexFeatureType::Vertex)
		{
			check(InOutFeatureIndex != INDEX_NONE);
			const int32 LocalVertexIndex0 = InOutFeatureIndex;
			const int32 VertexIndexA = Triangle.GetVertexIndex(LocalVertexIndex0);
			const FVec3& VertexA = Triangle.GetVertex(LocalVertexIndex0);

			// Loop over the fan of triangles that share VertexA
			// If the fan is not complete, we assume we have a boundary vertex and convert to a face collision
			int32 CurrentLocalTriangleIndex = LocalTriangleIndex;
			int32 NumTrianglesVisited = 0;
			while (true)
			{
				const FTriangleExt& CurrentTriangle = Triangles[CurrentLocalTriangleIndex];

				// Does the contact normal point into the infinite prism formed by extruding the triangle along the face normal?
				FVec3 CurrentVertexB, CurrentVertexC;
				if (CurrentTriangle.GetOtherVerticesFromID(VertexIndexA, CurrentVertexB, CurrentVertexC))
				{
					const FVec3& CurrentTriangleNormal = CurrentTriangle.GetNormal();
					const FVec3 CurrentEdge0 = CurrentVertexB - VertexA;
					const FVec3 CurrentEdge1 = CurrentVertexC - VertexA;
					const FVec3 CurrentEdgeNormal0 = FVec3::CrossProduct(CurrentTriangleNormal, CurrentEdge0);	// Not normalized
					const FVec3 CurrentEdgeNormal1 = FVec3::CrossProduct(CurrentEdge1, CurrentTriangleNormal);	// Not normalized
					const FReal CurrentEdgeSign0 = FVec3::DotProduct(InOutPlaneNormal, CurrentEdgeNormal0);		// Not normalized
					const FReal CurrentEdgeSign1 = FVec3::DotProduct(InOutPlaneNormal, CurrentEdgeNormal1);		// Not normalized
					const FReal CurrentEdgeSign0Sq = Utilities::SignedSquare(CurrentEdgeSign0);					// Not normalized
					const FReal CurrentEdgeSign1Sq = Utilities::SignedSquare(CurrentEdgeSign1);					// Not normalized

					// Scale tolerance to account for non-normalized sign
					const FReal SignToleranceSq = FReal(1.e-8);
					const FReal SignTolerance0Sq = SignToleranceSq * CurrentEdge0.SizeSquared();
					const FReal SignTolerance1Sq = SignToleranceSq * CurrentEdge1.SizeSquared();
					
					// If the normal is in the voronoi region of the triangle face, the only valid value for the normal is the face normal
					if ((CurrentEdgeSign0Sq >= FReal(-SignTolerance0Sq)) && (CurrentEdgeSign1Sq >= FReal(-SignTolerance1Sq)))
					{
						InOutFeatureType = Private::EConvexFeatureType::Plane;
						InOutFeatureIndex = 0;
						InOutPlaneNormal = CurrentTriangleNormal;
						return true;
					}

					// If the normal is in the voronoi region of the shared edge, use the triangle face normal
					// @todo(chaos): we should really use the two triangles sharing the edge to map the normal to a valid edge normal.
					if ((CurrentEdgeSign0Sq <= FReal(SignTolerance0Sq)) && (CurrentEdgeSign1Sq <= FReal(SignTolerance1Sq)))
					{
						const FReal NormalDotEdge0 = FVec3::DotProduct(InOutPlaneNormal, CurrentEdge0);
						const FReal NormalDotEdge1 = FVec3::DotProduct(InOutPlaneNormal, CurrentEdge1);
						if ((NormalDotEdge0 > 0) || (NormalDotEdge1 > 0))
						{
							InOutFeatureType = Private::EConvexFeatureType::Plane;
							InOutFeatureIndex = 0;
							InOutPlaneNormal = CurrentTriangleNormal;
							return true;
						}
					}
				}

				// Move to next triangle that shares edge A-B
				// NOTE: This relies on constent winding and that GetOtherVertexIDs returns indices in winding order
				int32 CurrentVertexIndexB, CurrentVertexIndexC;
				if (!ensure(CurrentTriangle.GetOtherVertexIDs(VertexIndexA, CurrentVertexIndexB, CurrentVertexIndexC)))
				{
					// Unexpected error - the triangle does not contain VertexA but it always should
					return false;
				}
				const int32 NextLocalTriangleIndex = GetOtherTriangleIndexForEdge(CurrentLocalTriangleIndex, FContactEdgeID(VertexIndexA, CurrentVertexIndexB));

				// Fan complete? We are done
				if (NextLocalTriangleIndex == LocalTriangleIndex)
				{
					break;
				}

				// Incomplete fan means boundary vertex. We treat boundary vertex contacts as a face contacts
				// NOTE: we may get false-positives here. Our triangle cache may not contain all the triangles that use
				// VertexA because the convex bounds does not overlap them all. However, if that happens we cannot
				// have a real vertex collision (if we overlap the vertex we should overlap all triangles that use it),
				// so the fallback should be reasonable even for false positives.
				if (NextLocalTriangleIndex == INDEX_NONE)
				{
					InOutFeatureType = Private::EConvexFeatureType::Plane;
					InOutFeatureIndex = 0;
					InOutPlaneNormal = TriangleNormal;
					return true;
				}


				CurrentLocalTriangleIndex = NextLocalTriangleIndex;
				++NumTrianglesVisited;

				// Malformed mesh?
				if (!ensure(NumTrianglesVisited < GetNumTriangles()))
				{
					InOutFeatureType = Private::EConvexFeatureType::Plane;
					InOutFeatureIndex = 0;
					InOutPlaneNormal = TriangleNormal;
					return true;
				}
			}
		}

		// We have nothing to do for face contacts
		return false;
	}

	void FMeshContactGenerator::SortContactByPhi()
	{
		Algo::Sort(Contacts,
			[](const FContactPoint& L, const FContactPoint& R)
			{
				return L.Phi < R.Phi;
			});
	}

	// Sort contacts on a shape pair in the order we like to solve them.
	// NOTE: This relies on the enum order of EContactPointType.
	void FMeshContactGenerator::SortContactsForSolverConvergence()
	{
		// Sort TriangleContactPoints in solver preferred order, but ignore TriangleContactPointDatas
		// NOTE: This should only be called at the end of the pruning proxess when we no longer care 
		// about TriangleContactPointDatas.
		if (Contacts.Num() > 1)
		{
			// Sort contact points by distance from the center of mass (RxN) so that points closer to the center of
			// mass are solved first. This produces better solver results for low iterations because, if we were to 
			// solve the distant points first, we would get extra rotation applied. 
			//
			// E.g., consider a box landing on an inclined plane with 5 contact points biassed toward one side. 
			//
			// -------------------
			// |                 |
			// |                 |
			// *-*-*-*-*----------
			//
			// Solving this left to right would result in extra clockwise rotation after 1 iteration. A subsequent
			// iteration would partially correct the problem.
			//
			TArray<TPair<FReal, int32>> SortKeyValues;
			SortKeyValues.SetNumUninitialized(Contacts.Num());
			for (int32 ContactIndex = 0; ContactIndex < Contacts.Num(); ++ContactIndex)
			{
				const FContactPoint& ContactPoint = Contacts[ContactIndex];
				const FVec3 DeltaTangent = ContactPoint.ShapeContactPoints[0] - FVec3::DotProduct(ContactPoint.ShapeContactPoints[0], ContactPoint.ShapeContactNormal) * ContactPoint.ShapeContactNormal;
				const FReal DeltaTangentLenSq = DeltaTangent.SizeSquared();
				SortKeyValues[ContactIndex] = { DeltaTangentLenSq, ContactIndex };
			}

			Algo::Sort(SortKeyValues,
				[](const TPair<FReal, int32>& L, const TPair<FReal, int32>& R)
				{
					return L.Key < R.Key;
				});

			TArray<FContactPoint> SortedContactPoints;
			SortedContactPoints.SetNumUninitialized(Contacts.Num());
			for (int32 ContactIndex = 0; ContactIndex < Contacts.Num(); ++ContactIndex)
			{
				SortedContactPoints[ContactIndex] = Contacts[SortKeyValues[ContactIndex].Value];
			}
			Swap(SortedContactPoints, Contacts);
		}
	}

	void FMeshContactGenerator::RemoveDisabledContacts()
	{
		// @todo(chaos): don't do this if we have not disabled any contacts

		// Re-pack the contact point array without re-ordering
		const int32 NumContactPoints = Contacts.Num();
		int32 DestContactIndex = 0;		// Index to where the next enabled item goes
		int32 SrcContactIndex = 0;		// Index to the next enabled item
		while (SrcContactIndex < NumContactPoints)
		{
			if (!ContactDatas[SrcContactIndex].IsEnabled())
			{
				// Find the next enabled point index to use as the copy source
				while (++SrcContactIndex < NumContactPoints)
				{
					if (ContactDatas[SrcContactIndex].IsEnabled())
					{
						break;
					}
				}

				if (SrcContactIndex == NumContactPoints)
				{
					break;
				}
			}

			// If we have removed elements, copy source to dest
			if (DestContactIndex != SrcContactIndex)
			{
				Contacts[DestContactIndex] = Contacts[SrcContactIndex];
				ContactDatas[DestContactIndex] = ContactDatas[SrcContactIndex];
			}

			++DestContactIndex;
			++SrcContactIndex;
		}

		// Clip the array to the enabled set
		Contacts.SetNum(DestContactIndex, EAllowShrinking::No);
		ContactDatas.SetNum(DestContactIndex, EAllowShrinking::No);
	}

	void FMeshContactGenerator::FinalizeContacts(const FRigidTransform3& MeshToConvexTransform)
	{
		for (int32 ContactIndex = 0; ContactIndex < Contacts.Num(); ++ContactIndex)
		{
			FContactPoint& ContactPoint = Contacts[ContactIndex];

			ContactPoint.ShapeContactPoints[1] = MeshToConvexTransform.InverseTransformPositionNoScale(ContactPoint.ShapeContactPoints[1]);
			ContactPoint.ShapeContactNormal = MeshToConvexTransform.InverseTransformVectorNoScale(ContactPoint.ShapeContactNormal);
		}
	}

	void FMeshContactGenerator::DebugDrawContacts(const FRigidTransform3& ConvexTransform, const FColor& Color, const FReal LineScale)
	{
#if CHAOS_DEBUG_DRAW
		if (CVars::ChaosSolverDebugDrawMeshContacts && FDebugDrawQueue::GetInstance().IsDebugDrawingEnabled())
		{
			check(LineScale > 0);
			const FReal Duration = 0;
			const int32 DrawPriority = 10;
			for (int32 ContactIndex = 0; ContactIndex < Contacts.Num(); ++ContactIndex)
			{
				const FTriangleContactPointData& ContactPointData = ContactDatas[ContactIndex];
				if (ContactPointData.IsEnabled())
				{
					const FContactPoint& ContactPoint = Contacts[ContactIndex];
					const FVec3 P1 = ConvexTransform.TransformPosition(ContactPoint.ShapeContactPoints[1]);
					const FVec3 N = ConvexTransform.TransformVectorNoScale(ContactPoint.ShapeContactNormal);

					// Draw the normal from the triangle face
					FDebugDrawQueue::GetInstance().DrawDebugLine(P1, P1 + LineScale * FReal(50) * N, Color, false, FRealSingle(Duration), DrawPriority, FRealSingle(1.0 / LineScale));
				}
			}
		}
#endif
	}

	void FMeshContactGenerator::DebugDrawTriangles(const FRigidTransform3& ConvexTransform, const FColor& VisitedColor, const FColor& IgnoredColor)
	{
#if CHAOS_DEBUG_DRAW
		if (CVars::ChaosSolverDebugDrawMeshContacts && FDebugDrawQueue::GetInstance().IsDebugDrawingEnabled())
		{
			// NOTE: drawing in two loops so that the visited triangle edges draw over the ignored ones (priority doesn't seem to work)
			for (int32 TriangleIndex = 0; TriangleIndex < Triangles.Num(); ++TriangleIndex)
			{
				const FTriangleExt& Triangle = Triangles[TriangleIndex];
				if (Triangle.GetVisitIndex() == INDEX_NONE)
				{
					DebugDrawTriangle(ConvexTransform, Triangle, IgnoredColor);
				}
			}
			for (int32 TriangleIndex = 0; TriangleIndex < Triangles.Num(); ++TriangleIndex)
			{
				const FTriangleExt& Triangle = Triangles[TriangleIndex];
				if (Triangle.GetVisitIndex() != INDEX_NONE)
				{
					DebugDrawTriangle(ConvexTransform, Triangle, VisitedColor);
				}
			}
		}
#endif
	}

	void FMeshContactGenerator::DebugDrawTriangle(const FRigidTransform3& ConvexTransform, const FTriangleExt& Triangle, const FColor& Color)
	{
#if CHAOS_DEBUG_DRAW
		const FReal Duration = 0;
		const FReal LineScale = 0.5;
		const int8 DrawPriority = 10;

		const FVec3 V0 = ConvexTransform.TransformPosition(Triangle.GetVertex(0));
		const FVec3 V1 = ConvexTransform.TransformPosition(Triangle.GetVertex(1));
		const FVec3 V2 = ConvexTransform.TransformPosition(Triangle.GetVertex(2));

		FDebugDrawQueue::GetInstance().DrawDebugLine(V0, V1, Color, false, FRealSingle(Duration), DrawPriority, FRealSingle(LineScale));
		FDebugDrawQueue::GetInstance().DrawDebugLine(V1, V2, Color, false, FRealSingle(Duration), DrawPriority, FRealSingle(LineScale));
		FDebugDrawQueue::GetInstance().DrawDebugLine(V2, V0, Color, false, FRealSingle(Duration), DrawPriority, FRealSingle(LineScale));
#endif
	}
}
