// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/Collision/PBDCollisionConstraint.h"
#include "Chaos/Collision/CollisionConstraintAllocator.h"
#include "Chaos/Collision/CollisionUtil.h"
#include "Chaos/Collision/PBDCollisionConstraintHandle.h"
#include "Chaos/Collision/PBDCollisionContainerSolver.h"
#include "Chaos/Collision/PBDCollisionSolver.h"
#include "Chaos/CollisionResolution.h"
#include "Chaos/Evolution/SolverBody.h"
#include "Chaos/Evolution/SolverBodyContainer.h"
#include "Chaos/Island/IslandManager.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/Particle/ParticleUtilities.h"
#include "Chaos/PBDCollisionConstraints.h"


//UE_DISABLE_OPTIMIZATION

namespace Chaos
{
	namespace CVars
	{
		extern FRealSingle CCDEnableThresholdBoundsScale;
		extern FRealSingle CCDAllowedDepthBoundsScale;
	}

	extern bool bChaos_Collision_EnableManifoldGJKInject;
	extern bool bChaos_Collision_EnableManifoldGJKReplace;

	FRealSingle Chaos_Manifold_MatchPositionTolerance = 0.3f;		// Fraction of object size position tolerance
	FRealSingle Chaos_Manifold_MatchNormalTolerance = 0.02f;		// Dot product tolerance
	FAutoConsoleVariableRef CVarChaos_Manifold_MatchPositionTolerance(TEXT("p.Chaos.Collision.Manifold.MatchPositionTolerance"), Chaos_Manifold_MatchPositionTolerance, TEXT("A tolerance as a fraction of object size used to determine if two contact points are the same"));
	FAutoConsoleVariableRef CVarChaos_Manifold_MatchNormalTolerance(TEXT("p.Chaos.Collision.Manifold.MatchNormalTolerance"), Chaos_Manifold_MatchNormalTolerance, TEXT("A tolerance on the normal dot product used to determine if two contact points are the same"));

	FRealSingle Chaos_Manifold_FrictionExactPositionTolerance = 0.2f;	// Distance a shape-relative contact point can move and we can still be sure it's the same point (don't look for better options)
	FRealSingle Chaos_Manifold_FrictionNearPositionTolerance = 1.0f;	// Distance a shape-relative contact point can move and still be considered the same point, if no better option exists
	FAutoConsoleVariableRef CVarChaos_Manifold_FrictionExactPositionTolerance(TEXT("p.Chaos.Collision.Manifold.FrictionExactPositionTolerance"), Chaos_Manifold_FrictionExactPositionTolerance, TEXT(""));
	FAutoConsoleVariableRef CVarChaos_Manifold_FrictionNearPositionTolerance(TEXT("p.Chaos.Collision.Manifold.FrictionNearPositionTolerance"), Chaos_Manifold_FrictionNearPositionTolerance, TEXT(""));


	FRealSingle Chaos_GBFCharacteristicTimeRatio = 1.0f;
	FAutoConsoleVariableRef CVarChaos_GBFCharacteristicTimeRatio(TEXT("p.Chaos.Collision.GBFCharacteristicTimeRatio"), Chaos_GBFCharacteristicTimeRatio, TEXT("The ratio between characteristic time and Dt"));

	bool bChaos_Manifold_EnableFrictionRestore = true;
	FAutoConsoleVariableRef CVarChaos_Manifold_EnableFrictionRestore(TEXT("p.Chaos.Collision.Manifold.EnableFrictionRestore"), bChaos_Manifold_EnableFrictionRestore, TEXT(""));
	
	// The margin to use when we are colliding a convex shape against a zero-margin shape. E.g., Box-Triangle.
	// When both shapes have a margin we use the minimum margin, but we don't want to use a zero margin because we hit the EPA degenerate case
	// NOTE: This is currently disabled - margins for convex-trimesh cause bigger problems than the EPA issue
	FRealSingle Chaos_Collision_ConvexZeroMargin = 0.0f;
	FAutoConsoleVariableRef CVarChaos_Collision_ConvexZeroMargin(TEXT("p.Chaos.Collision.ConvexZeroMargin"), Chaos_Collision_ConvexZeroMargin, TEXT(""));

	// A collision solver stiffness override if >= 0. A value < 0 means use ths config setting.
	FRealSingle Chaos_Collision_Stiffness = -1.0f;
	FAutoConsoleVariableRef CVarChaos_Collision_Stiffness(TEXT("p.Chaos.Collision.Stiffness"), Chaos_Collision_Stiffness, TEXT("Override the collision solver stiffness (if >= 0)"));

	// Enable/Disable the one-way interaction system for collisions
	bool bChaos_Collision_EnableOneWayInteraction = true;
	FAutoConsoleVariableRef CVarChaos_Collision_EnableOneWayInteraction(TEXT("p.Chaos.Collision.EnableOneWayInteraction"), bChaos_Collision_EnableOneWayInteraction, TEXT("Whether the one-way interaction flag is respected in collision constraints"));

	// The stiffness used for collision between one-way objects and other dynamic objects. The stiffness against kinematics and statics is not affected. A value of less than 1
	// will treat collisions with the static scene as "harder" than collisions with dynamics which will prevent one-way objects from getting squeezed out of the world when a dynamic
	// lands on top of them. Instead, they will stay on top of the terrain and sink into the dynamic.
	FRealSingle Chaos_Collision_OneWayStiffness = 0.5f;
	FAutoConsoleVariableRef CVarChaos_Collision_OneWayStiffness(TEXT("p.Chaos.Collision.OneWayStiffness"), Chaos_Collision_OneWayStiffness, TEXT("Collision solver stiffnes for one-way interactions"));

	bool bChaos_Collision_EnableBoundsChecks = true;
	FAutoConsoleVariableRef CVarChaos_Collision_EnableBoundsChecks(TEXT("p.Chaos.Collision.EnableBoundsChecks"), bChaos_Collision_EnableBoundsChecks, TEXT(""));

	// Maximum number of manifold points per contact (-1 for unlimited)
	int32 Chaos_Collision_MaxManifoldPoints = -1;
	FAutoConsoleVariableRef CVarChaos_Collision_MaxManifoldPoints(TEXT("p.Chaos.Collision.MaxManifoldPoints"), Chaos_Collision_MaxManifoldPoints, TEXT(""));

	bool bChaos_Collision_AllowGlobalInitialPhi = true;
	FAutoConsoleVariableRef CVarChaos_Collision_AllowGlobalInitialPhi(TEXT("p.Chaos.Collision.AllowGlobalInitialPhi"), bChaos_Collision_AllowGlobalInitialPhi, TEXT(""));

	bool bChaos_Collision_SimpleAssignContact = true;
	FAutoConsoleVariableRef CVarChaos_Collision_SimpleAssignContacti(TEXT("p.Chaos.Collision.SimpleAssignContact"), bChaos_Collision_SimpleAssignContact, TEXT("Try to assign contact without worrying if the contact has been already assigned. Improve performance and ease to sleep when enabled."));


	struct FCollisionTolerances
	{
		// Multiplied by the contact margin to produce a distance within which contacts are considered to be the same point
		FReal ContactPositionToleranceScale = FReal(0.8);

		// Multiplied by the contact margin to produce a max distance that a shape can move if we want to reuse contacts
		FReal ShapePositionToleranceScale0 = FReal(0.5);	// 0 contacts
		FReal ShapePositionToleranceScaleN = FReal(0.2);	// >0 contacts

		// A threshold on the quaternion change the tells us when we cannot reuse contacts
		FReal ShapeRotationThreshold0 = FReal(0.9998);		// 0 contacts
		FReal ShapeRotationThresholdN = FReal(0.9999);		// >0 contacts

		// Thresholds used to restore individual manifold points
		FReal ManifoldPointPositionToleranceScale = FReal(1);
		FReal ManifoldPointNormalThreshold  = FReal(0.7);
	};

	// @todo(chaos): put these tolerances on cvars
	// @todo(chaos): tune the tolerances used in FPBDCollisionConstraint::TryRestoreManifold
	FCollisionTolerances Chaos_Manifold_Tolerances;

	FString FPBDCollisionConstraint::ToString() const
	{
		return FString::Printf(TEXT("Particle:%s, Levelset:%s, AccumulatedImpulse:%s"), *Particle[0]->ToString(), *Particle[1]->ToString(), *AccumulatedImpulse.ToString());
	}

	bool ContactConstraintSortPredicate(const FPBDCollisionConstraint& L, const FPBDCollisionConstraint& R)
	{
		//sort constraints by the smallest particle idx in them first
		//if the smallest particle idx is the same for both, use the other idx

		if (L.GetCCDEnabled() != R.GetCCDEnabled())
		{
			return L.GetCCDEnabled();
		}

		const FParticleID ParticleIdxs[] = { L.Particle[0]->ParticleID(), L.Particle[1]->ParticleID() };
		const FParticleID OtherParticleIdxs[] = { R.Particle[0]->ParticleID(), R.Particle[1]->ParticleID() };

		const int32 MinIdx = ParticleIdxs[0] < ParticleIdxs[1] ? 0 : 1;
		const int32 OtherMinIdx = OtherParticleIdxs[0] < OtherParticleIdxs[1] ? 0 : 1;

		if(ParticleIdxs[MinIdx] < OtherParticleIdxs[OtherMinIdx])
		{
			return true;
		} 
		else if(ParticleIdxs[MinIdx] == OtherParticleIdxs[OtherMinIdx])
		{
			return ParticleIdxs[!MinIdx] < OtherParticleIdxs[!OtherMinIdx];
		}

		return false;
	}

	void FPBDCollisionConstraint::Make(
		FGeometryParticleHandle* Particle0,
		const FImplicitObject* Implicit0,
		const FPerShapeData* Shape0,
		const FBVHParticles* Simplicial0,
		const FRigidTransform3& ImplicitLocalTransform0,
		FGeometryParticleHandle* Particle1,
		const FImplicitObject* Implicit1,
		const FPerShapeData* Shape1,
		const FBVHParticles* Simplicial1,
		const FRigidTransform3& ImplicitLocalTransform1,
		const FReal InCullDistance,
		const bool bInUseManifold,
		const EContactShapesType ShapesType,
		FPBDCollisionConstraint& OutConstraint)
	{
		OutConstraint.Particle[0] = Particle0;
		OutConstraint.Particle[1] = Particle1;
		OutConstraint.Implicit[0] = Implicit0;
		OutConstraint.Implicit[1] = Implicit1;
		OutConstraint.Shape[0] = (Shape0 != nullptr) ? Shape0->AsShapeInstance() : nullptr;
		OutConstraint.Shape[1] = (Shape1 != nullptr) ? Shape1->AsShapeInstance() : nullptr;
		OutConstraint.Simplicial[0] = Simplicial0;
		OutConstraint.Simplicial[1] = Simplicial1;

		OutConstraint.Setup(ECollisionCCDType::Disabled, ShapesType, ImplicitLocalTransform0, ImplicitLocalTransform1, InCullDistance, bInUseManifold);
	}

	FPBDCollisionConstraint FPBDCollisionConstraint::MakeTriangle(const FImplicitObject* Implicit0)
	{
		FPBDCollisionConstraint Constraint;
		Constraint.InitMarginsAndTolerances(Implicit0->GetCollisionType(), ImplicitObjectType::Triangle, Implicit0->GetMarginf(), FRealSingle(0));
		return Constraint;
	}

	// Used by the resim system. Constraints copied this way will later be returned via RestoreFrom()
	FPBDCollisionConstraint FPBDCollisionConstraint::MakeCopy(const FPBDCollisionConstraint& Source)
	{
		// @todo(chaos): The resim cache version probably doesn't need all the data, so maybe try to cut this down?
		FPBDCollisionConstraint Constraint = Source;

		// Invalidate the data that maps the constraint to its container (we are no longer in the container)
		// @todo(chaos): this should probably be handled by the copy constructor
		Constraint.GetContainerCookie().ClearContainerData();

		// We are not in the constraint graph, even if Source was
		Constraint.SetConstraintGraphEdge(nullptr);

		return Constraint;
	}

	// Used by the resim system. Restore a constraint from the past that we saved with MakeCopy
	void FPBDCollisionConstraint::RestoreFrom(const FPBDCollisionConstraint& Source)
	{
		// We do not want to overwrite these properties because they are managed by the systems
		// where the constraint is registsred (allocator, container, graph).
		const FPBDCollisionConstraintContainerCookie Cookie = ContainerCookie;
		const Private::FCollisionSortKey SortKey = CollisionSortKey;
		Private::FPBDIslandConstraint* ConstraintGraphEdge = GetConstraintGraphEdge();

		// Copy everything
		*this = Source;

		// Restore the system state
		ContainerCookie = Cookie;
		CollisionSortKey = SortKey;
		SetConstraintGraphEdge(ConstraintGraphEdge);
	}

	FPBDCollisionConstraint::FPBDCollisionConstraint()
		: CollisionMargins{ 0, 0 }
		, ImplicitTransform{ FRigidTransform3(), FRigidTransform3() }
		, Particle{ nullptr, nullptr }
		, Implicit{ nullptr, nullptr }
		, Shape{ nullptr, nullptr }
		, Simplicial{ nullptr, nullptr }
		, AccumulatedImpulse(0)
		, CullDistance(TNumericLimits<FRealSingle>::Max())
		, RelativeMovement(0)
		, ContainerCookie()
		, CollisionTolerance(0)
		, ClosestManifoldPointIndex(INDEX_NONE)
		, ShapeWorldTransforms{ FRigidTransform3(), FRigidTransform3() }
		, ExpectedNumManifoldPoints(0)
		, LastShapeWorldPositionDelta()
		, LastShapeWorldRotationDelta()
		, GJKWarmStartData()
		, Material()
		, Stiffness(1)
		, Flags()
		, ShapesType(EContactShapesType::Unknown)
		, SavedManifoldPoints()
		, ManifoldPoints()
		, MinInitialPhi(0)
		, InitialOverlapDepenetrationVelocity(-1.0f)
		, CCDTimeOfImpact(0)
		, SolverBodies{ nullptr, nullptr }
		, CCDEnablePenetration(0)
		, CCDTargetPenetration(0)
	{
	}

	FPBDCollisionConstraint::FPBDCollisionConstraint(
		FGeometryParticleHandle* Particle0,
		const FImplicitObject* Implicit0,
		const FPerShapeData* Shape0,
		const FBVHParticles* Simplicial0,
		FGeometryParticleHandle* Particle1,
		const FImplicitObject* Implicit1,
		const FPerShapeData* Shape1,
		const FBVHParticles* Simplicial1)
		: CollisionMargins{ 0, 0 }
		, ImplicitTransform{ FRigidTransform3(), FRigidTransform3() }
		, Particle{ Particle0, Particle1 }
		, Implicit{ Implicit0, Implicit1 }
		, Shape{ nullptr, nullptr }
		, Simplicial{ Simplicial0, Simplicial1 }
		, AccumulatedImpulse(0)
		, CullDistance(TNumericLimits<FRealSingle>::Max())
		, RelativeMovement(0)
		, ContainerCookie()
		, CollisionTolerance(0)
		, ClosestManifoldPointIndex(INDEX_NONE)
		, ShapeWorldTransforms{ FRigidTransform3(), FRigidTransform3() }
		, ExpectedNumManifoldPoints(0)
		, LastShapeWorldPositionDelta()
		, LastShapeWorldRotationDelta()
		, GJKWarmStartData()
		, Material()
		, Stiffness(1)
		, Flags()
		, ShapesType(EContactShapesType::Unknown)
		, SavedManifoldPoints()
		, ManifoldPoints()
		, MinInitialPhi(0)
		, InitialOverlapDepenetrationVelocity(-1.0f)
		, CCDTimeOfImpact(0)
		, SolverBodies{ nullptr, nullptr }
		, CCDEnablePenetration(0)
		, CCDTargetPenetration(0)
	{
		// These downcasts should never fail if the input shape is not null
		// @todo(chaos): remove when FPerShapeData is deprecated
		check((Shape0 == nullptr) || (Shape0->AsShapeInstance() != nullptr));
		check((Shape1 == nullptr) || (Shape1->AsShapeInstance() != nullptr));
		if (Shape0 != nullptr)
		{
			Shape[0] = Shape0->AsShapeInstance();
		}
		if (Shape1 != nullptr)
		{
			Shape[1] = Shape1->AsShapeInstance();
		}
	}

	FPBDCollisionConstraint::~FPBDCollisionConstraint()
	{
#if !UE_BUILD_TEST && !UE_BUILD_SHIPPING
		// Make sure we have been dropped by the graph
		ensure(GetConstraintGraphEdge() == nullptr);
#endif
	}

	void FPBDCollisionConstraint::Setup(
		const ECollisionCCDType InCCDType,
		const EContactShapesType InShapesType,
		const FRigidTransform3& InImplicitLocalTransform0,
		const FRigidTransform3& InImplicitLocalTransform1,
		const FReal InCullDistance,
		const bool bInUseManifold)
	{
		ShapesType = InShapesType;

		ImplicitTransform[0] = InImplicitLocalTransform0;
		ImplicitTransform[1] = InImplicitLocalTransform1;

		CullDistance = FRealSingle(InCullDistance);
		RelativeMovement = FVec3(0);

		FRealSingle DistanceCheckSize = 0;
		BoundsTestFlags = Private::CalculateImplicitBoundsTestFlags(
			GetParticle0(), GetImplicit0(), GetShape0(),
			GetParticle1(), GetImplicit1(), GetShape1(),
			DistanceCheckSize);

		// The constraint will be enabled after the narrow phase if required
		Flags.bDisabled = true;

		// Contact modification may change bIsProbe. The original value is stored in BoundsTestFlags.
		Flags.bIsProbe = BoundsTestFlags.bIsProbe;

		// If this currently a CCD contact (this may change on later ticks)
		Flags.bCCDEnabled = (InCCDType == ECollisionCCDType::Enabled);

		// Initialize tolerances that depend on shape type etc, also the bIsQuadratic flags
		const FRealSingle Margin0 = GetImplicit0()->GetMarginf();
		const FRealSingle Margin1 = GetImplicit1()->GetMarginf();
		const EImplicitObjectType ImplicitType0 = Private::GetImplicitCollisionType(GetParticle0(), GetImplicit0());
		const EImplicitObjectType ImplicitType1 = Private::GetImplicitCollisionType(GetParticle1(), GetImplicit1());
		InitMarginsAndTolerances(ImplicitType0, ImplicitType1, Margin0, Margin1);

		// Are we allowing manifolds?
		Flags.bUseManifold = bInUseManifold;
		Flags.bUseIncrementalManifold = false;

		// The initial overlap depenetration velocity is that max of the two body settings. I.e., if either body wants to depenetrate, they will.
		const FRealSingle InitialOverlapDepenetrationVelocity0 = FConstGenericParticleHandle(GetParticle0())->InitialOverlapDepenetrationVelocity();
		const FRealSingle InitialOverlapDepenetrationVelocity1 = FConstGenericParticleHandle(GetParticle1())->InitialOverlapDepenetrationVelocity();
		InitialOverlapDepenetrationVelocity = FMath::Max3(InitialOverlapDepenetrationVelocity0, InitialOverlapDepenetrationVelocity1, 0.0f);

		// Is this a one-way interaction? A dynamic one-way interaction particle that hits a kinematic non one-way interaction particle should still be considered one-way.
		const bool bDynamic0 = FConstGenericParticleHandle(GetParticle0())->IsDynamic();
		const bool bDynamic1 = FConstGenericParticleHandle(GetParticle1())->IsDynamic();
		const bool bOneWay0 = FConstGenericParticleHandle(GetParticle0())->OneWayInteraction();
		const bool bOneWay1 = FConstGenericParticleHandle(GetParticle1())->OneWayInteraction();
		Flags.bIsOneWayInteraction = (bDynamic0 || bDynamic1) && (bOneWay0 || bOneWay1);

		// For dynamic against kinematic (as long as no spheres or capsules are involved) we share initial overlap between all manifold points.
		// This is to make convex-mesh collisions work better for large overlaps on complicated meshes, but it is not ideal.
		// @todo(chaos): try to remove this
		Flags.bUsePerContactInitialPhi = true;
		if (bChaos_Collision_AllowGlobalInitialPhi && !Flags.bIsQuadratic0 && !Flags.bIsQuadratic1 && (!bDynamic0 || !bDynamic1))
		{
			Flags.bUsePerContactInitialPhi = false;
		}

		// Only levelsets use incremental collision manifolds
		if (bInUseManifold && ((ImplicitType0 == ImplicitObjectType::LevelSet) || (ImplicitType1 == ImplicitObjectType::LevelSet)))
		{
			Flags.bUseIncrementalManifold = true;
		}
	}

	void FPBDCollisionConstraint::InitMarginsAndTolerances(const EImplicitObjectType ImplicitType0, const EImplicitObjectType ImplicitType1, const FRealSingle Margin0, const FRealSingle Margin1)
	{
		// Set up the margins and tolerances to be used during the narrow phase.
		// When convex margins are enabled, at least one shape in a collision will always have a margin.
		// If convex margins are disabled, only quadratic shapes have a margin (their radius).
		// The collision tolerance is used for knowing whether a new contact matches an existing one.
		//
		// Margins: (Assuming convex margins are enabled...)
		// If a polygonal shape is not dynamic, its margin is zero.
		// Triangles always have zero margin (they are never dynamic anyway)
		// If we have two polygonal shapes, we use the smallest of the two margins unless one shape has zero margin.
		// If we have a quadratic shape versus a polygonal shape, we use a zero margin on the polygonal shape.
		// Note: If we have a triangle, it is always the second shape (currently we do not support triangle-triangle collision)
		//
		// CollisionTolerance:
		// For polygonal shapes the collision tolerance is proportional to the size of the smaller object. 
		// For quadratic shapes we want a collision tolerance much smaller than the radius.
		//
		const bool bIsQuadratic0 = ((ImplicitType0 == ImplicitObjectType::Sphere) || (ImplicitType0 == ImplicitObjectType::Capsule));
		const bool bIsQuadratic1 = ((ImplicitType1 == ImplicitObjectType::Sphere) || (ImplicitType1 == ImplicitObjectType::Capsule));

		// @todo(chaos): should probably be tunable. Used to use the same settings as the margin scale (for convex), but we want to support zero
		// margins, but still have a non-zero CollisionTolerance (it is used for matching contact points for friction and manifold reuse)
		const FRealSingle ToleranceScale = 0.1f;
		const FRealSingle QuadraticToleranceScale = 0.05f;
		
		if (!bIsQuadratic0 && !bIsQuadratic1)
		{
			// Only dynamic polytopes objects use margins
			const bool bIsDynamic0 = (GetParticle0() != nullptr) && (FConstGenericParticleHandle(GetParticle0())->IsDynamic());
			const bool bIsDynamic1 = (GetParticle1() != nullptr) && (FConstGenericParticleHandle(GetParticle1())->IsDynamic());
			const FRealSingle DynamicMargin0 = bIsDynamic0 ? FRealSingle(Margin0) : FRealSingle(0);
			const FRealSingle DynamicMargin1 = bIsDynamic1 ? FRealSingle(Margin1) : FRealSingle(0);

			const FRealSingle MaxSize0 = ((Implicit[0] != nullptr) && Implicit[0]->HasBoundingBox()) ? FRealSingle(Implicit[0]->BoundingBox().Extents().GetAbsMax()) : FRealSingle(0);
			const FRealSingle MaxSize1 = ((Implicit[1] != nullptr) && Implicit[1]->HasBoundingBox()) ? FRealSingle(Implicit[1]->BoundingBox().Extents().GetAbsMax()) : FRealSingle(0);
			const FRealSingle MaxSize = FMath::Min(MaxSize0, MaxSize1);
			CollisionTolerance = ToleranceScale * MaxSize;

			// If one shape has a zero margin, enforce a minimum margin to avoid the EPA degenerate case. E.g., Box-Triangle
			// If both shapes have a margin, use the smaller margin on both shapes. E.g., Box-Box
			// We should never see both shapes with zero margin, but if we did we'd end up with a zero margin
			const FRealSingle MinMargin = Chaos_Collision_ConvexZeroMargin;
			if (DynamicMargin0 == FRealSingle(0))
			{
				CollisionMargins[0] = 0;
				CollisionMargins[1] = FMath::Max(MinMargin, DynamicMargin1);
			}
			else if (DynamicMargin1 == FRealSingle(0))
			{
				CollisionMargins[0] = FMath::Max(MinMargin, DynamicMargin0);
				CollisionMargins[1] = 0;
			}
			else
			{
				CollisionMargins[0] = FMath::Min(DynamicMargin0, DynamicMargin1);
				CollisionMargins[1] = CollisionMargins[0];
			}
		}
		else if (bIsQuadratic0 && bIsQuadratic1)
		{
			CollisionMargins[0] = Margin0;
			CollisionMargins[1] = Margin1;
			CollisionTolerance = QuadraticToleranceScale * FMath::Min(Margin0, Margin1);
		}
		else if (bIsQuadratic0 && !bIsQuadratic1)
		{
			CollisionMargins[0] = Margin0;
			CollisionMargins[1] = 0;
			CollisionTolerance = QuadraticToleranceScale * Margin0;
		}
		else if (!bIsQuadratic0 && bIsQuadratic1)
		{
			CollisionMargins[0] = 0;
			CollisionMargins[1] = Margin1;
			CollisionTolerance = QuadraticToleranceScale * Margin1;
		}

		Flags.bIsQuadratic0 = bIsQuadratic0;
		Flags.bIsQuadratic1 = bIsQuadratic1;
	}

	void FPBDCollisionConstraint::InitCCDThreshold()
	{
		// Calculate the depth at which we enforce CCD constraints, and the amount of penetration to leave when resolving a CCD constraint.
		const FRealSingle MinBounds0 = FConstGenericParticleHandle(Particle[0])->CCDEnabled() ? FRealSingle(Implicit[0]->BoundingBox().Extents().GetAbsMin()) : FRealSingle(0);
		const FRealSingle MinBounds1 = FConstGenericParticleHandle(Particle[1])->CCDEnabled() ? FRealSingle(Implicit[1]->BoundingBox().Extents().GetAbsMin()) : FRealSingle(0);
		const FRealSingle MinBounds = FMath::Max(MinBounds0, MinBounds1);

		// If we perform a CCD sweep and it finds a hit, but the the penetration depth at the end of
		// the sweep (T=1) is less than this, we ignore the sweep result (no rewind) and just run
		// regular collision detection at T=1 (i.e., as if CCD is not enabled for this tick).
		// If this is zero, then we will always perform CCD rewind when we get a sweep hit.
		CCDEnablePenetration = MinBounds * CVars::CCDEnableThresholdBoundsScale;
		
		// If we have a CCD sweep hit above the enable penetration, we will rewind the body so
		// that the penetration depth CCDTargetPenetration (or less). A new contact manifold is then calculated.
		// NOTE: this can be zero but if it is we will get very poor behaviour because there will be
		// nothing for the contact resolution to do since we fully depenetrated in the CCD rewind.
		CCDTargetPenetration = MinBounds * CVars::CCDAllowedDepthBoundsScale;
	}

	void FPBDCollisionConstraint::Activate()
	{
		Flags.bIsCurrent = true;
		Flags.bDisabled = false;

		// Match new manifold points with data from previous tick
		AssignSavedManifoldPoints();

		// If any materials were changed, update the friction etc
		UpdateMaterialProperties();

		// Undo any temporary modifications from user callbacks
		ResetModifications();

		AccumulatedImpulse = FVec3(0);
	}

	void FPBDCollisionConstraint::UpdateMaterialPropertiesImpl()
	{
		ConcreteContainer()->UpdateConstraintMaterialProperties(*this);

		UpdateMassScales();
	}

	void FPBDCollisionConstraint::UpdateMassScales()
	{
		if (Flags.bIsOneWayInteraction && bChaos_Collision_EnableOneWayInteraction)
		{
			const bool bOneWay0 = FConstGenericParticleHandle(GetParticle0())->OneWayInteraction();
			const bool bOneWay1 = FConstGenericParticleHandle(GetParticle1())->OneWayInteraction();
			if (bOneWay0 && !bOneWay1)
			{
				Material.InvMassScale1 = 0;
				Material.InvInertiaScale1 = 0;
			}
			if (bOneWay1 && !bOneWay0)
			{
				Material.InvMassScale0 = 0;
				Material.InvInertiaScale0 = 0;
			}
			Stiffness = Chaos_Collision_OneWayStiffness;
		}
		else
		{
			Material.InvMassScale0 = 1;
			Material.InvInertiaScale0 = 1;
			Material.InvMassScale1 = 1;
			Material.InvInertiaScale1 = 1;
			Stiffness = 1;

			// Debug testing for solver stiffness
			if (Chaos_Collision_Stiffness >= 0)
			{
				SetStiffness(Chaos_Collision_Stiffness);
			}
		}
	}

	bool FPBDCollisionConstraint::IsSleeping() const
	{
		if (ContainerCookie.MidPhase != nullptr)
		{
			return ContainerCookie.MidPhase->IsSleeping();
		}
		return false;
	}

	void FPBDCollisionConstraint::SetIsSleeping(const bool bInIsSleeping)
	{
		// This actually sets the sleeping state on all constraints between the same particle pair so calling this with multiple
		// constraints on the same particle pair is a little wasteful. It early-outs on subsequent calls, but still not ideal.
		// @todo(chaos): we only need to set sleeping on particle pairs or particles, not constraints (See UpdateSleepState in IslandManager.cpp)
		check(ContainerCookie.MidPhase != nullptr);
		ContainerCookie.MidPhase->SetIsSleeping(bInIsSleeping, ConcreteContainer()->GetConstraintAllocator().GetCurrentEpoch());
	}

	Chaos::FVec3f FPBDCollisionConstraint::GetNetLinearImpulse() const
	{
		FVec3f TotalNetImpulse = FVec3f(0, 0, 0);
		for (const FManifoldPointResult& ManifoldPoint : ManifoldPointResults)
		{
			TotalNetImpulse += ManifoldPoint.NetImpulse;
		}
		return TotalNetImpulse;
	}

	void FPBDCollisionConstraint::UpdateParticleTransform(FGeometryParticleHandle* InParticle)
	{
		// We need to update the friction anchors based on how much the user moved the particle, but
		// it's easier to just reset the friction state altogether, and possibly the expected behaviour anyway
		ResetManifold();
	}

	FVec3 FPBDCollisionConstraint::CalculateWorldContactLocation() const
	{
		if (ClosestManifoldPointIndex != INDEX_NONE)
		{
			const FVec3 WorldContact0 = GetShapeWorldTransform0().TransformPositionNoScale(FVec3(ManifoldPoints[ClosestManifoldPointIndex].ContactPoint.ShapeContactPoints[0]));
			const FVec3 WorldContact1 = GetShapeWorldTransform1().TransformPositionNoScale(FVec3(ManifoldPoints[ClosestManifoldPointIndex].ContactPoint.ShapeContactPoints[1]));
			return FReal(0.5) * (WorldContact0 + WorldContact1);
		}
		return FVec3(0);
	}

	FVec3 FPBDCollisionConstraint::CalculateWorldContactNormal() const
	{
		if (ClosestManifoldPointIndex != INDEX_NONE)
		{
			return GetShapeWorldTransform1().TransformVectorNoScale(FVec3(ManifoldPoints[ClosestManifoldPointIndex].ContactPoint.ShapeContactNormal));
		}
		return FVec3(0, 0, 1);
	}

	// Are the two manifold points the same point?
	// Ideally a contact is considered the same as one from the previous iteration if
	//		The contact is Vertex - Face and there was a prior iteration collision on the same Vertex
	//		The contact is Edge - Edge and a prior iteration collision contained both edges
	//		The contact is Face - Face and a prior iteration contained both faces
	//
	// But we don’t have feature IDs. So in the meantime contact points will be considered the "same" if
	//		Vertex - Face - the local space contact position on either body is within some tolerance
	//		Edge - Edge - ?? hard...
	//		Face - Face - ?? hard...
	//
	bool FPBDCollisionConstraint::AreMatchingContactPoints(const FContactPoint& A, const FContactPoint& B, FReal& OutScore) const
	{
		OutScore = 0.0f;

		// @todo(chaos): cache tolerances?
		FReal DistanceTolerance = 0.0f;
		if (Particle[0]->GetGeometry()->HasBoundingBox() && Particle[1]->GetGeometry()->HasBoundingBox())
		{
			const FReal Size0 = Particle[0]->GetGeometry()->BoundingBox().Extents().Max();
			const FReal Size1 = Particle[1]->GetGeometry()->BoundingBox().Extents().Max();
			DistanceTolerance = FMath::Min(Size0, Size1) * Chaos_Manifold_MatchPositionTolerance;
		}
		else if (Particle[0]->GetGeometry()->HasBoundingBox())
		{
			const FReal Size0 = Particle[0]->GetGeometry()->BoundingBox().Extents().Max();
			DistanceTolerance = Size0 * Chaos_Manifold_MatchPositionTolerance;
		}
		else if (Particle[1]->GetGeometry()->HasBoundingBox())
		{
			const FReal Size1 = Particle[1]->GetGeometry()->BoundingBox().Extents().Max();
			DistanceTolerance = Size1 * Chaos_Manifold_MatchPositionTolerance;
		}
		else
		{
			return false;
		}
		const FReal NormalTolerance = Chaos_Manifold_MatchNormalTolerance;

		// If normal has changed a lot, it is a different contact
		// (This was only here to detect bad normals - it is not right for edge-edge contact tracking, but we don't do a good job of that yet anyway!)
		FReal NormalDot = FVec3::DotProduct(A.ShapeContactNormal, B.ShapeContactNormal);
		if (NormalDot < 1.0f - NormalTolerance)
		{
			return false;
		}

		// If either point in local space is the same, it is the same contact
		if (DistanceTolerance > 0.0f)
		{
			const FReal DistanceTolerance2 = DistanceTolerance * DistanceTolerance;
			for (int32 BodyIndex = 0; BodyIndex < 2; ++BodyIndex)
			{
				FVec3 DR = A.ShapeContactPoints[BodyIndex] - B.ShapeContactPoints[BodyIndex];
				FReal DRLen2 = DR.SizeSquared();
				if (DRLen2 < DistanceTolerance2)
				{
					OutScore = FMath::Clamp(1.f - DRLen2 / DistanceTolerance2, 0.f, 1.f);
					return true;
				}
			}
		}

		return false;
	}

	int32 FPBDCollisionConstraint::FindManifoldPoint(const FContactPoint& ContactPoint) const
	{
		const int32 NumManifoldPoints = ManifoldPoints.Num();
		int32 BestMatchIndex = INDEX_NONE;
		FReal BestMatchScore = 0.0f;
		for (int32 ManifoldPointIndex = 0; ManifoldPointIndex < NumManifoldPoints; ++ManifoldPointIndex)
		{
			FReal Score = 0.0f;
			if (AreMatchingContactPoints(ContactPoint, ManifoldPoints[ManifoldPointIndex].ContactPoint, Score))
			{
				if (Score > BestMatchScore)
				{
					BestMatchIndex = ManifoldPointIndex;
					BestMatchScore = Score;

					// Just take the first one that meets the tolerances
					break;
				}
			}
		}
		return BestMatchIndex;
	}

	void FPBDCollisionConstraint::UpdateManifoldPointPhi(const int32 ManifoldPointIndex)
	{
		FManifoldPoint& ManifoldPoint = ManifoldPoints[ManifoldPointIndex];
		const FVec3 WorldContact0 = GetShapeWorldTransform0().TransformPositionNoScale(FVec3(ManifoldPoint.ContactPoint.ShapeContactPoints[0]));
		const FVec3 WorldContact1 = GetShapeWorldTransform1().TransformPositionNoScale(FVec3(ManifoldPoint.ContactPoint.ShapeContactPoints[1]));
		const FVec3 WorldContactNormal = GetShapeWorldTransform1().TransformVectorNoScale(FVec3(ManifoldPoint.ContactPoint.ShapeContactNormal));
		ManifoldPoint.ContactPoint.Phi = FRealSingle(FVec3::DotProduct(WorldContact0 - WorldContact1, WorldContactNormal));
	}

	void FPBDCollisionConstraint::CorrectManifoldPoints()
	{
		const FConstGenericParticleHandle P0 = Particle[0];
		const FConstGenericParticleHandle P1 = Particle[1];
		const FReal InvM0 = P0->InvM();
		const FReal InvM1 = P1->InvM();
		const FReal Alpha0 = ((InvM0 + InvM1) > FReal(0)) ? InvM1 / (InvM0 + InvM1) : FReal(0);
		const FReal Alpha1 = FReal(1) - Alpha0;

		for (int32 ManifoldPointIndex = 0; ManifoldPointIndex < ManifoldPoints.Num(); ManifoldPointIndex++)
		{
			FManifoldPoint& ManifoldPoint = ManifoldPoints[ManifoldPointIndex];
			const FVec3 WorldContact0 = GetShapeWorldTransform0().TransformPositionNoScale(FVec3(ManifoldPoint.ContactPoint.ShapeContactPoints[0]));
			const FVec3 WorldContact1 = GetShapeWorldTransform1().TransformPositionNoScale(FVec3(ManifoldPoint.ContactPoint.ShapeContactPoints[1]));
			const FVec3 WorldContactNormal = GetShapeWorldTransform1().TransformVectorNoScale(FVec3(ManifoldPoint.ContactPoint.ShapeContactNormal));
			const FVec3 WorldContactDelta = WorldContact1 - WorldContact0;
			const FReal WorldContactDeltaNormal = FVec3::DotProduct(WorldContactDelta, WorldContactNormal);
			const FVec3 WorldContactDeltaTangent = WorldContactDelta - WorldContactDeltaNormal * WorldContactNormal;

			const FVec3 CorrectedWorldContact0 = WorldContact0 + Alpha0 * WorldContactDeltaTangent;
			const FVec3 CorrectedWorldContact1 = WorldContact1 - Alpha1 * WorldContactDeltaTangent;
			FContactPoint CorrectedContactPoint = ManifoldPoint.ContactPoint;
			CorrectedContactPoint.ShapeContactPoints[0] = GetShapeWorldTransform0().InverseTransformPositionNoScale(CorrectedWorldContact0);
			CorrectedContactPoint.ShapeContactPoints[1] = GetShapeWorldTransform1().InverseTransformPositionNoScale(CorrectedWorldContact1);
			CorrectedContactPoint.Phi = -WorldContactDeltaNormal;

			InitManifoldPoint(ManifoldPointIndex, CorrectedContactPoint);
		}
	}

	void FPBDCollisionConstraint::UpdateManifoldContacts()
	{
		if ((GetSolverBody0() != nullptr) && (GetSolverBody1() != nullptr))
		{
			// This is only entered when calling collision detection in a legacy solver (RBAN).
			// We need to update the contact Phi for the current iteration based on what the body transforms would be if
			// we applied the corrections accumulated so far.
			// @todo(chaos): It is extremely expensicve! Remove this when RBAN uses the QuasiPBD solver.
			// NOTE: ShapeRelativeTransforms are in actor-space. The SolverBodies give CoM transforms.
			const FConstGenericParticleHandle P0 = Particle[0];
			const FConstGenericParticleHandle P1 = Particle[1];

			const FRigidTransform3 ParticleCoMTransform0 = FRigidTransform3(
				GetSolverBody0()->CorrectedP(), 
				GetSolverBody0()->CorrectedQ());

			const FRigidTransform3 ParticleCoMTransform1 = FRigidTransform3(
				GetSolverBody1()->CorrectedP(), 
				GetSolverBody1()->CorrectedQ());

			const FRigidTransform3 ShapeCoMRelativeTransform0 = FRigidTransform3(
				P0->RotationOfMass().UnrotateVector(ImplicitTransform[0].GetLocation() - P0->CenterOfMass()),
				P0->RotationOfMass().Inverse() * ImplicitTransform[0].GetRotation());

			const FRigidTransform3 ShapeCoMRelativeTransform1 = FRigidTransform3(
				P1->RotationOfMass().UnrotateVector(ImplicitTransform[1].GetLocation() - P1->CenterOfMass()),
				P1->RotationOfMass().Inverse() * ImplicitTransform[1].GetRotation());

			ShapeWorldTransforms[0] = ShapeCoMRelativeTransform0 * ParticleCoMTransform0;
			ShapeWorldTransforms[1] = ShapeCoMRelativeTransform1 * ParticleCoMTransform1;
		}

		ClosestManifoldPointIndex = INDEX_NONE;

		for (int32 ManifoldPointIndex = 0; ManifoldPointIndex < ManifoldPoints.Num(); ManifoldPointIndex++)
		{
			FManifoldPoint& ManifoldPoint = ManifoldPoints[ManifoldPointIndex];

			UpdateManifoldPointPhi(ManifoldPointIndex);

			if (ManifoldPoint.ContactPoint.Phi < GetPhi())
			{
				ClosestManifoldPointIndex = ManifoldPointIndex;
			}
		}
	}

	void FPBDCollisionConstraint::AddIncrementalManifoldContact(const FContactPoint& ContactPoint)
	{
		if (Flags.bUseManifold)
		{
			// See if the manifold point already exists
			int32 ManifoldPointIndex = FindManifoldPoint(ContactPoint);
			if (ManifoldPointIndex >= 0)
			{
				// This contact point is already in the manifold - update the state
				ManifoldPoints[ManifoldPointIndex].ContactPoint = ContactPoint;
			}
			else
			{
				// This is a new manifold point - capture the state and generate initial properties
				ManifoldPointIndex = AddManifoldPoint(ContactPoint);
			}

			// If collision detection did its job, this contact is the deepest
			// NOTE: other contact Phis will be out of date at the current iteration's transforms
			ClosestManifoldPointIndex = ManifoldPointIndex;
		}
		else 
		{
			// We are not using manifolds - reuse the first and only point
			ManifoldPoints.SetNum(1);

			InitManifoldPoint(0, ContactPoint);

			ClosestManifoldPointIndex = 0;
		}
	}

	void FPBDCollisionConstraint::ResetManifold()
	{
		ResetSavedManifoldPoints();
		ResetActiveManifoldContacts();
	}

	void FPBDCollisionConstraint::ResetActiveManifoldContacts()
	{
		ClosestManifoldPointIndex = INDEX_NONE;
		ManifoldPoints.Reset();
		ManifoldPointResults.Reset();
		ExpectedNumManifoldPoints = 0;
		Flags.bWasManifoldRestored = false;
		Flags.bCanRestoreManifold = false;
	}

	bool FPBDCollisionConstraint::TryRestoreManifold()
	{
		const FCollisionTolerances Tolerances = FCollisionTolerances();//Chaos_Manifold_Tolerances;
		const FReal ContactPositionTolerance = Tolerances.ContactPositionToleranceScale * CollisionTolerance;
		const FReal ShapePositionTolerance = (ManifoldPoints.Num() > 0) ? Tolerances.ShapePositionToleranceScaleN * CollisionTolerance : Tolerances.ShapePositionToleranceScale0 * CollisionTolerance;
		const FReal ShapeRotationThreshold = (ManifoldPoints.Num() > 0) ? Tolerances.ShapeRotationThresholdN : Tolerances.ShapeRotationThreshold0;
		const FReal ContactPositionToleranceSq = FMath::Square(ContactPositionTolerance);

		// Reset current closest point
		ClosestManifoldPointIndex = INDEX_NONE;
		Flags.bWasManifoldRestored = false;

		if (!Flags.bCanRestoreManifold || Flags.bModifierApplied)
		{
			return false;
		}

		// If we have not moved or rotated much we may reuse some of the manifold points, as long as they have not moved far as well (see below)
		bool bMovedBeyondTolerance = true;
		if ((ShapePositionTolerance > 0) && (ShapeRotationThreshold > 0))
		{
			// The transform check is necessary regardless of how many points we have left in the manifold because
			// as a body moves/rotates we may have to change which faces/edges are colliding. We can't know if the face/edge
			// will change until we run the closest-point checks (GJK) in the narrow phase.
			const FVec3 Shape1ToShape0Translation = ShapeWorldTransforms[0].GetTranslation() - ShapeWorldTransforms[1].GetTranslation();
			const FVec3 TranslationDelta = Shape1ToShape0Translation - LastShapeWorldPositionDelta;
			if (TranslationDelta.IsNearlyZero(ShapePositionTolerance))
			{
				const FRotation3 Shape1toShape0Rotation = ShapeWorldTransforms[0].GetRotation().Inverse() * ShapeWorldTransforms[1].GetRotation();
				const FRotation3 OriginalShape1toShape0Rotation = LastShapeWorldRotationDelta;
				const FReal RotationOverlap = FRotation3::DotProduct(Shape1toShape0Rotation, LastShapeWorldRotationDelta);
				if (RotationOverlap > ShapeRotationThreshold)
				{
					bMovedBeyondTolerance = false;
				}
			}
		}

		if (bMovedBeyondTolerance)
		{
			return false;
		}

		// Either update or remove each manifold point depending on how far it has moved from its initial relative point
		// NOTE: We do not reset if we have 0 points - we can still "restore" a zero point manifold if the bodies have not moved
		if (ManifoldPoints.Num() > 0)
		{
			const FRigidTransform3 Shape0ToShape1Transform = ShapeWorldTransforms[0].GetRelativeTransformNoScale(ShapeWorldTransforms[1]);
			
			// Update or prune manifold points
			int32 NumActiveManifoldPoints = ManifoldPoints.Num();
			for (int32 ManifoldPointIndex = 0; ManifoldPointIndex < ManifoldPoints.Num(); ++ManifoldPointIndex)
			{
				FManifoldPoint& ManifoldPoint = ManifoldPoints[ManifoldPointIndex];
				
				// Point may have been disabled on a previous tick
				if (ManifoldPoint.Flags.bDisabled)
				{
					--NumActiveManifoldPoints;
					continue;
				}

				// Calculate the world-space contact location and separation at the current shape transforms
				// @todo(chaos): this should use the normal owner. Currently we assume body 1 is the owner
				const FVec3 Contact0In1 = Shape0ToShape1Transform.TransformPositionNoScale(FVec3(ManifoldPoint.InitialShapeContactPoints[0]));
				const FVec3& Contact1In1 = ManifoldPoint.InitialShapeContactPoints[1];
				const FVec3& ContactNormalIn1 = ManifoldPoint.ContactPoint.ShapeContactNormal;

				const FVec3 ContactDeltaIn1 = Contact0In1 - Contact1In1;
				const FReal ContactPhi = FVec3::DotProduct(ContactDeltaIn1, ContactNormalIn1);
				const FVec3 ContactLateralDeltaIn1 = ContactDeltaIn1 - ContactPhi * ContactNormalIn1;
				const FReal ContactLateralDistanceSq = ContactLateralDeltaIn1.SizeSquared();

				// Either update the point or flag it for removal
				if (ContactLateralDistanceSq < ContactPositionToleranceSq)
				{
					// Recalculate the contact points at the new location
					// @todo(chaos): we should reproject the contact on the plane owner
					const FVec3f ShapeContactPoint1 = FVec3f(Contact0In1 - ContactPhi * ContactNormalIn1);
					ManifoldPoint.ContactPoint.ShapeContactPoints[1] = ShapeContactPoint1;
					ManifoldPoint.ContactPoint.Phi = FRealSingle(ContactPhi);

					ManifoldPoint.Flags.bWasRestored = true;
					ManifoldPoint.Flags.bWasReplaced = false;
					ManifoldPoint.Flags.bInitialContact = false;

					if (ManifoldPoint.ContactPoint.Phi < GetPhi())
					{
						ClosestManifoldPointIndex = ManifoldPointIndex;
					}
				}
				else
				{
					// This point moved too far - disable it
					ManifoldPoint.Flags.bDisabled = true;
					ManifoldPoint.Flags.bWasRestored = false;
					ManifoldPoint.Flags.bWasReplaced = false;
					ManifoldPoint.Flags.bInitialContact = false;
					--NumActiveManifoldPoints;
				}
			}

			// We want to retain a stable supporting manifold, so at least 4 points. Unless we started with less than that, 
			// in which case make sure it doesn't go any lower.
			ExpectedNumManifoldPoints = FMath::Min(ManifoldPoints.Num(), 4);

			// If we removed too many points, we must rebuild the manifold
			// NOTE: We can only add up to one point
			if (NumActiveManifoldPoints < ExpectedNumManifoldPoints)
			{
				return false;
			}
		}

		Flags.bWasManifoldRestored = true;
		return true;
	}

	bool FPBDCollisionConstraint::TryAddManifoldContact(const FContactPoint& NewContactPoint)
	{
		const FCollisionTolerances& Tolerances = Chaos_Manifold_Tolerances;
		const FReal PositionTolerance = Tolerances.ManifoldPointPositionToleranceScale * CollisionTolerance;
		const FReal NormalThreshold = Tolerances.ManifoldPointNormalThreshold;

		// We must end up with a full manifold after this if we want to reuse it
		//if ((ManifoldPoints.Num() < ExpectedNumManifoldPoints - 1) || (ExpectedNumManifoldPoints == 0))
		//{
		//	// We need to add more than 1 point to restore the manifold so we must rebuild it from scratch
		//	return false;
		//}
		if (ManifoldPoints.Num() == 0)
		{
			return false;
		}

		// Find the matching manifold point if it exists and replace it
		// Also check to see if the normal has changed significantly and if it has force manifold regeneration
		// NOTE: the normal rejection check assumes all contacts have the same normal - this may not always be true. The worst
		// case here is that we will regenerate the manifold too often so it will work but could be bad for perf
		const FReal PositionToleranceSq = FMath::Square(PositionTolerance);
		int32 MatchedManifoldPointIndex = INDEX_NONE;
		for (int32 ManifoldPointIndex = 0; ManifoldPointIndex < ManifoldPoints.Num(); ++ManifoldPointIndex)
		{
			FManifoldPoint& ManifoldPoint = ManifoldPoints[ManifoldPointIndex];

			const FReal NormalOverlap = FVec3::DotProduct(ManifoldPoint.ContactPoint.ShapeContactNormal, NewContactPoint.ShapeContactNormal);
			if (NormalOverlap < NormalThreshold)
			{
				return false;
			}

			const FVec3 DR0 = ManifoldPoint.ContactPoint.ShapeContactPoints[0] - NewContactPoint.ShapeContactPoints[0];
			const FVec3 DR1 = ManifoldPoint.ContactPoint.ShapeContactPoints[1] - NewContactPoint.ShapeContactPoints[1];
			if ((DR0.SizeSquared() < PositionToleranceSq) && (DR1.SizeSquared() < PositionToleranceSq))
			{
				// If the existing point has a deeper penetration, just re-use it. This is common when we have a GJK
				// result on an edge or corner - the contact created when generating the manifold is on the
				// surface shape rather than the rounded (margin-reduced) shape.
				// If the new point is deeper, use it.
				if (ManifoldPoint.ContactPoint.Phi > NewContactPoint.Phi)
				{
					ManifoldPoint.ContactPoint = NewContactPoint;
					ManifoldPoint.InitialShapeContactPoints[0] = NewContactPoint.ShapeContactPoints[0];
					ManifoldPoint.InitialShapeContactPoints[1] = NewContactPoint.ShapeContactPoints[1];
					ManifoldPoint.Flags.bWasRestored = false;
					ManifoldPoint.Flags.bWasReplaced = true;
					if (NewContactPoint.Phi < GetPhi())
					{
						ClosestManifoldPointIndex = ManifoldPointIndex;
					}
				}

				return true;
			}
		}

		// If we have a full manifold, see if we can use or reject the GJK point
		if ((ManifoldPoints.Num() == 4) && bChaos_Collision_EnableManifoldGJKInject)
		{
			return TryInsertManifoldContact(NewContactPoint);
		}
		
		return false;
	}

	bool FPBDCollisionConstraint::TryInsertManifoldContact(const FContactPoint& NewContactPoint)
	{
		check(ManifoldPoints.Num() == 4);

		const int32 NormalBodyIndex = 1;
		constexpr int32 NumContactPoints = 5;
		constexpr int32 NumManifoldPoints = 4;

		// We want to select 4 points from the 5 we have
		// Create a working set of points, and keep track which points have been selected
		FVec3 ContactPoints[NumContactPoints];
		FReal ContactPhis[NumContactPoints];
		bool bContactSelected[NumContactPoints];
		int32 SelectedContactIndices[NumManifoldPoints];
		for (int32 ContactIndex = 0; ContactIndex < NumManifoldPoints; ++ContactIndex)
		{
			const FManifoldPoint& ManifoldPoint = ManifoldPoints[ContactIndex];
			ContactPoints[ContactIndex] = ManifoldPoint.ContactPoint.ShapeContactPoints[NormalBodyIndex];
			ContactPhis[ContactIndex] = ManifoldPoint.ContactPoint.Phi;
			bContactSelected[ContactIndex] = false;
		}
		ContactPoints[4] = NewContactPoint.ShapeContactPoints[NormalBodyIndex];
		ContactPhis[4] = NewContactPoint.Phi;
		bContactSelected[4] = false;

		// We are projecting points into a plane perpendicular to the contact normal, which we assume is the new point's normal
		const FVec3 ContactNormal = NewContactPoint.ShapeContactNormal;

		// Start with the deepest point. This may not be point 4 despite that being the result of
		// collision detection because for some shape types we use margin-reduced core shapes which
		// are effectively rounded at the corners. But...when building a one-shot manifold we 
		// use the outer shape to get sharp corners. So, if we have a GJK result from a "corner"
		// the real corner (if it is in the manifold) may actually be deeper than the GJK result.
		SelectedContactIndices[0] = 0;
		for (int32 ContactIndex = 1; ContactIndex < NumContactPoints; ++ContactIndex)
		{
			if (ContactPhis[ContactIndex] < ContactPhis[SelectedContactIndices[0]])
			{
				SelectedContactIndices[0] = ContactIndex;
			}
		}
		bContactSelected[SelectedContactIndices[0]] = true;

		// The second point will be the one farthest from the first
		SelectedContactIndices[1] = INDEX_NONE;
		FReal MaxDistanceSq = TNumericLimits<FReal>::Lowest();
		for (int32 ContactIndex = 0; ContactIndex < NumContactPoints; ++ContactIndex)
		{
			if (!bContactSelected[ContactIndex])
			{
				const FReal DistanceSq = (ContactPoints[ContactIndex] - ContactPoints[SelectedContactIndices[0]]).SizeSquared();
				if (DistanceSq > MaxDistanceSq)
				{
					SelectedContactIndices[1] = ContactIndex;
					MaxDistanceSq = DistanceSq;
				}
			}
		}
		check(SelectedContactIndices[1] != INDEX_NONE);
		bContactSelected[SelectedContactIndices[1]] = true;

		// The third point is the one which gives us the largest triangle (projected onto a plane perpendicular to the normal)
		SelectedContactIndices[2] = INDEX_NONE;
		FReal MaxTriangleArea = 0;
		FReal WindingOrder = FReal(1.0);
		for (int32 ContactIndex = 0; ContactIndex < NumContactPoints; ++ContactIndex)
		{
			if (!bContactSelected[ContactIndex])
			{
				const FVec3 Cross = FVec3::CrossProduct(ContactPoints[SelectedContactIndices[1]] - ContactPoints[SelectedContactIndices[0]], ContactPoints[ContactIndex] - ContactPoints[SelectedContactIndices[1]]);
				const FReal SignedArea = FVec3::DotProduct(Cross, ContactNormal);
				if (FMath::Abs(SignedArea) > MaxTriangleArea)
				{
					SelectedContactIndices[2] = ContactIndex;
					MaxTriangleArea = FMath::Abs(SignedArea);
					WindingOrder = FMath::Sign(SignedArea);
				}
			}
		}
		if (SelectedContactIndices[2] == INDEX_NONE)
		{
			// Degenerate points - all 4 exactly in a line
			return false;
		}
		bContactSelected[SelectedContactIndices[2]] = true;

		// The fourth point is the one which adds the most area to the 3 points we already have
		SelectedContactIndices[3] = INDEX_NONE;
		FReal MaxQuadArea = 0;	// Additional area to MaxTriangleArea
		for (int32 ContactIndex = 0; ContactIndex < NumContactPoints; ++ContactIndex)
		{
			if (!bContactSelected[ContactIndex])
			{
				// Calculate the area that is added by inserting the point into each edge of the selected triangle
				// The signed area will be negative for interior points, positive for points that extend the triangle into a quad.
				const FVec3 Cross0 = FVec3::CrossProduct(ContactPoints[ContactIndex] - ContactPoints[SelectedContactIndices[0]], ContactPoints[SelectedContactIndices[1]] - ContactPoints[ContactIndex]);
				const FReal SignedArea0 = WindingOrder * FVec3::DotProduct(Cross0, ContactNormal);
				const FVec3 Cross1 = FVec3::CrossProduct(ContactPoints[ContactIndex] - ContactPoints[SelectedContactIndices[1]], ContactPoints[SelectedContactIndices[2]] - ContactPoints[ContactIndex]);
				const FReal SignedArea1 = WindingOrder * FVec3::DotProduct(Cross1, ContactNormal);
				const FVec3 Cross2 = FVec3::CrossProduct(ContactPoints[ContactIndex] - ContactPoints[SelectedContactIndices[2]], ContactPoints[SelectedContactIndices[0]] - ContactPoints[ContactIndex]);
				const FReal SignedArea2 = WindingOrder * FVec3::DotProduct(Cross2, ContactNormal);
				const FReal SignedArea = FMath::Max3(SignedArea0, SignedArea1, SignedArea2);
				if (SignedArea > MaxQuadArea)
				{
					SelectedContactIndices[3] = ContactIndex;
					MaxQuadArea = SignedArea;
				}
			}
		}
		if (SelectedContactIndices[3] == INDEX_NONE)
		{
			// No point is outside the triangle we already have
			return false;
		}
		bContactSelected[SelectedContactIndices[3]] = true;

		// Now we should have exactly 4 selected contacts. If we find that one of the existing points is not
		// selected, it must be because it is being replaced by the new contact. Otherwise the new contact
		// is interior to the existing manifiold and is rejected.
		for (int32 ManifoldPointIndex = 0; ManifoldPointIndex < NumManifoldPoints; ++ManifoldPointIndex)
		{
			if (!bContactSelected[ManifoldPointIndex])
			{
				ManifoldPoints[ManifoldPointIndex].ContactPoint = NewContactPoint;
				ManifoldPoints[ManifoldPointIndex].InitialShapeContactPoints[0] = NewContactPoint.ShapeContactPoints[0];
				ManifoldPoints[ManifoldPointIndex].InitialShapeContactPoints[1] = NewContactPoint.ShapeContactPoints[1];
				ManifoldPoints[ManifoldPointIndex].Flags.bWasRestored = false;
				if (NewContactPoint.Phi < GetPhi())
				{
					ClosestManifoldPointIndex = ManifoldPointIndex;
				}
			}
		}

		return true;
	}

	FRealSingle FPBDCollisionConstraint::CalculateSavedManifoldPointDistanceSq(const FSavedManifoldPoint& SavedManifoldPoint, const FManifoldPoint& ManifoldPoint) const
	{
		// If we have a vertex-plane (or vertex-vertex) contact, we want to know if we have the same vertex(es).
		// If we have and edge-edge contact, we want to know if we have the same edges.
		// But we don't know what type of contact we have, so for now...
		// If the contact point is in the same spot on one of the bodies, assume it is the same contact
		// @todo(chaos) - collision detection should provide the contact point types (vertex/edge/plane)
		const FVec3f DP0 = ManifoldPoint.ContactPoint.ShapeContactPoints[0] - SavedManifoldPoint.ShapeContactPoints[0];
		const FVec3f DP1 = ManifoldPoint.ContactPoint.ShapeContactPoints[1] - SavedManifoldPoint.ShapeContactPoints[1];

		// When only one shape is quadratic, we only look at the quadratic contact point so we don't identify
		// a sphere spinning on the spot as a stationary contact
		// @todo(chaos): we don't really need the quadratic special case any more
		if (IsQuadratic0() && !IsQuadratic1())
		{
			return DP0.SizeSquared();
		}
		else if (IsQuadratic1() && !IsQuadratic0())
		{
			return DP1.SizeSquared();
		}
		else
		{
			return FMath::Min(DP0.SizeSquared(), DP1.SizeSquared());
		}
	}

	int32 FPBDCollisionConstraint::FindSimpleSavedManifoldPoint(const int32 ManifoldPointIndex) const
	{
		if (bChaos_Manifold_EnableFrictionRestore)
		{
			const FManifoldPoint& ManifoldPoint = ManifoldPoints[ManifoldPointIndex];
			if (!ManifoldPoint.Flags.bDisabled)
			{
				const int32 NumAllowedSavedPoints = SavedManifoldPoints.Num();
				const FRealSingle ExactDistanceToleranceSq = FMath::Square(Chaos_Manifold_FrictionExactPositionTolerance);
				const FRealSingle NearDistanceToleranceSq = FMath::Square(Chaos_Manifold_FrictionNearPositionTolerance);

				// Ignore points farther than NearDistanceTolerance
				FRealSingle BestDistanceSq = NearDistanceToleranceSq;

				int32 MatchAllowedPointIndex = INDEX_NONE;
				for (int32 PointIndex = 0; PointIndex < NumAllowedSavedPoints; ++PointIndex)
				{
					const int32 MatchPointIndex = (PointIndex + ManifoldPointIndex) % NumAllowedSavedPoints;
					const FSavedManifoldPoint& SavedManifoldPoint = SavedManifoldPoints[MatchPointIndex];
					const FRealSingle DistanceSq = CalculateSavedManifoldPointDistanceSq(SavedManifoldPoint, ManifoldPoint);

					// If this is an exact match, take the point without searching further
					if (DistanceSq < ExactDistanceToleranceSq)
					{
						BestDistanceSq = DistanceSq;
						MatchAllowedPointIndex = MatchPointIndex;
						break;
					}

					// If this is a close match, we may use it if we can't find a better one
					if (DistanceSq < BestDistanceSq)
					{
						BestDistanceSq = DistanceSq;
						MatchAllowedPointIndex = MatchPointIndex;
					}
				}
					return MatchAllowedPointIndex;
			}
		}
		return INDEX_NONE;
	}

	int32 FPBDCollisionConstraint::FindSavedManifoldPoint(const int32 ManifoldPointIndex, int32* InOutAllowedSavedPointIndices, int32& InOutNumAllowedSavedPoints) const
	{
		if (bChaos_Manifold_EnableFrictionRestore)
		{
			const FManifoldPoint& ManifoldPoint = ManifoldPoints[ManifoldPointIndex];
			if (!ManifoldPoint.Flags.bDisabled)
			{
				const FRealSingle ExactDistanceToleranceSq = FMath::Square(Chaos_Manifold_FrictionExactPositionTolerance);
				const FRealSingle NearDistanceToleranceSq = FMath::Square(Chaos_Manifold_FrictionNearPositionTolerance);

				// Ignore popints farther than NearDistanceTolerance
				FRealSingle BestDistanceSq = NearDistanceToleranceSq;

				int32 MatchAllowedPointIndex = INDEX_NONE;
				for (int32 AllowedPointIndex = 0; AllowedPointIndex < InOutNumAllowedSavedPoints; ++AllowedPointIndex)
				{
					const int32 SavedPointIndex = InOutAllowedSavedPointIndices[AllowedPointIndex];
					const FSavedManifoldPoint& SavedManifoldPoint = SavedManifoldPoints[SavedPointIndex];

					const FRealSingle DistanceSq = CalculateSavedManifoldPointDistanceSq(SavedManifoldPoint, ManifoldPoint);

					// If this is an exact match, take the point without searching further
					if (DistanceSq < ExactDistanceToleranceSq)
					{
						BestDistanceSq = DistanceSq;
						MatchAllowedPointIndex = AllowedPointIndex;
						break;
					}

					// If this is a close match, we may use it if we can't find a better one
					if (DistanceSq < BestDistanceSq)
					{
						BestDistanceSq = DistanceSq;
						MatchAllowedPointIndex = AllowedPointIndex;
					}
				}

				// If we selected a point, remove it from the set
				if (MatchAllowedPointIndex != INDEX_NONE)
				{
					// RemoveAtSwap
					const int32 MatchIndex = InOutAllowedSavedPointIndices[MatchAllowedPointIndex];
					--InOutNumAllowedSavedPoints;
					if (MatchAllowedPointIndex < InOutNumAllowedSavedPoints)
					{
						InOutAllowedSavedPointIndices[MatchAllowedPointIndex] = InOutAllowedSavedPointIndices[InOutNumAllowedSavedPoints];
					}
					return MatchIndex;
				}
			}
		}
		return INDEX_NONE;
	}

	void FPBDCollisionConstraint::AssignSavedManifoldPoints()
	{
		TManifoldPointArray<int32> AllowedPointIndices;
		int32 NumAllowedPointIndices = 0;
		if (!bChaos_Collision_SimpleAssignContact)
		{
			AllowedPointIndices.SetNum(SavedManifoldPoints.Num());
			for (int32 PointIndex = 0, PointEndIndex = SavedManifoldPoints.Num(); PointIndex < PointEndIndex; ++PointIndex)
			{
				AllowedPointIndices[PointIndex] = PointIndex;
			}
			NumAllowedPointIndices = AllowedPointIndices.Num();
		}

		for (int32 PointIndex = 0, PointEndIndex = ManifoldPoints.Num(); PointIndex < PointEndIndex; ++PointIndex)
		{
			// @todo(chaos): ideally we would skip this step for restored manifold points but for now we need to 
			// extract InitialPhi from the results
			// @see UpdateCollisionSolverContactPointFromConstraint
			//if (!ManifoldPoints[PointIndex].Flags.bWasRestored)
			{
				// Assume this is a new contact
				ManifoldPoints[PointIndex].Flags.bInitialContact = true;

				// See if we have saved data for this contact
				int32 SavedManifoldPointIndex = INDEX_NONE;
				if (bChaos_Collision_SimpleAssignContact)
				{
					SavedManifoldPointIndex = FindSimpleSavedManifoldPoint(PointIndex);
				}
				else
				{
					SavedManifoldPointIndex = FindSavedManifoldPoint(PointIndex, AllowedPointIndices.GetData(), NumAllowedPointIndices);
				}
				if (SavedManifoldPointIndex != INDEX_NONE)
				{
					ManifoldPoints[PointIndex].Flags.bHasStaticFrictionAnchor = true;
					ManifoldPoints[PointIndex].ShapeAnchorPoints[0] = SavedManifoldPoints[SavedManifoldPointIndex].ShapeContactPoints[0];
					ManifoldPoints[PointIndex].ShapeAnchorPoints[1] = SavedManifoldPoints[SavedManifoldPointIndex].ShapeContactPoints[1];
					ManifoldPoints[PointIndex].InitialPhi = SavedManifoldPoints[SavedManifoldPointIndex].InitialPhi;
					ManifoldPoints[PointIndex].Flags.bInitialContact = false;
				}
				// Nothing to do if no saved friction point because we already set the anchor to the most recently detected contact point
				// (See InitManifoldPoint, And TryRestoreManifold)
			}
		}
	}

	ECollisionConstraintDirection FPBDCollisionConstraint::GetConstraintDirection(const FReal Dt) const
	{
		if (GetDisabled() || GetIsProbe())
		{
			return NoRestingDependency;
		}
		// D\tau is the chacteristic time (as in GBF paper Sec 8.1)
		const FReal Dtau = Dt * Chaos_GBFCharacteristicTimeRatio; 

		const FVec3 Normal = CalculateWorldContactNormal();
		const FReal Phi = GetPhi();
		if (GetPhi() >= GetCullDistance())
		{
			return NoRestingDependency;
		}

		FVec3 GravityDirection = ConcreteContainer()->GetGravityDirection();
		FReal GravitySize = ConcreteContainer()->GetGravitySize();
		// When gravity is zero, we still want to sort the constraints instead of having a random order. In this case, set gravity to default gravity.
		if (GravitySize < UE_SMALL_NUMBER)
		{
			GravityDirection = FVec3(0, 0, -1);
			GravitySize = 980.f;
		}

		// How far an object travels in gravity direction within time Dtau starting with zero velocity (as in GBF paper Sec 8.1). 
		// Theoretically this should be 0.5 * GravityMagnitude * Dtau * Dtau.
		// Omitting 0.5 to be more consistent with our integration scheme.
		// Multiplying 0.5 can alternatively be achieved by setting Chaos_GBFCharacteristicTimeRatio=sqrt(0.5)
		const FReal StepSize = GravitySize * Dtau * Dtau; 
		const FReal NormalDotG = FVec3::DotProduct(Normal, GravityDirection);
		const FReal NormalDirectionThreshold = 0.1f; // Hack
		if (NormalDotG < -NormalDirectionThreshold) // Object 0 rests on object 1
		{
			if (Phi + NormalDotG * StepSize < 0) // Hack to simulate object 0 falling (as in GBF paper Sec 8.1)
			{
				return Particle1ToParticle0;
			}
			else
			{
				return NoRestingDependency;
			}
		}
		else if (NormalDotG > NormalDirectionThreshold) // Object 1 rests on object 0
		{
			if (Phi - NormalDotG * StepSize < 0) // Hack to simulate object 1 falling (as in GBF paper Sec 8.1)
			{
				return Particle0ToParticle1;
			}
			else
			{
				return NoRestingDependency;
			}
		}
		else // Horizontal contact
		{
			return NoRestingDependency;
		}
	}

	void FPBDCollisionConstraint::LogOneShotManifoldError(const int32 MaxManifoldPoints, const TArrayView<const FContactPoint>& ContactPoints)
	{
		UE_LOGF(LogChaos, Error, "FPBDCollisionConstraint: exceeded per-constraint manifold point limit at %d / %d. Particles %ls -:- %ls", ContactPoints.Num(), MaxManifoldPoints, *GetParticle0()->GetDebugName(), *GetParticle1()->GetDebugName());
	}
}