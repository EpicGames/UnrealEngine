// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/Collision/PBDCollisionConstraintHandle.h"
#include "Chaos/Collision/PBDCollisionConstraint.h"
#include "Chaos/Evolution/SolverBody.h"
#include "Chaos/Evolution/SolverBodyContainer.h"
#include "Chaos/PBDCollisionConstraints.h"

namespace Chaos
{

	const FConstraintHandleTypeID& FPBDCollisionConstraintHandle::StaticType()
	{
		static FConstraintHandleTypeID STypeID(TEXT("FCollisionConstraintHandle"), &FIntrusiveConstraintHandle::StaticType());
		return STypeID;
	}
}