// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/Collision/PBDCollisionContainerSolver.h"

#include "Chaos/Collision/CollisionApplyType.h"
#include "Chaos/Collision/PBDCollisionConstraint.h"
#include "Chaos/Collision/PBDCollisionSolverUtilities.h"
#include "Chaos/CollisionResolution.h"
#include "Chaos/DebugDrawQueue.h"
#include "Chaos/Evolution/SolverBodyContainer.h"
#include "Chaos/Island/IslandManager.h"
#include "Chaos/Particle/ParticleUtilities.h"
#include "Chaos/PBDCollisionConstraints.h"
#include "Chaos/PBDCollisionConstraintsContact.h"
#include "Chaos/Utilities.h"
#include "Templates/AlignmentTemplates.h"

// Private includes

#include "ChaosLog.h"

namespace Chaos
{
	namespace CVars
	{
		extern bool bChaos_PBDCollisionSolver_Position_SolveEnabled;
		extern bool bChaos_PBDCollisionSolver_Velocity_SolveEnabled;
		extern float Chaos_PBDCollisionSolver_Position_MinInvMassScale;
		extern float Chaos_PBDCollisionSolver_Velocity_MinInvMassScale;
		extern bool bChaos_PBDCollisionSolver_Use1DFriction;

		// If one body is more than MassRatio1 times the mass of the other, adjust the solver stiffness when the lighter body is underneath.
		// Solver stiffness will be equal to 1 when the mass ratio is MassRatio1.
		// Solver stiffness will be equal to 0 when the mass ratio is MassRatio2.
		FRealSingle Chaos_PBDCollisionSolver_AutoStiffness_MassRatio1 = 0;
		FRealSingle Chaos_PBDCollisionSolver_AutoStiffness_MassRatio2 = 0;
		FAutoConsoleVariableRef CVarChaosPBDCollisionSolverAutoStiffnessMassRatio1(TEXT("p.Chaos.PBDCollisionSolver.AutoStiffness.MassRatio1"), Chaos_PBDCollisionSolver_AutoStiffness_MassRatio1, TEXT(""));
		FAutoConsoleVariableRef CVarChaosPBDCollisionSolverAutoStiffnessMassRatio2(TEXT("p.Chaos.PBDCollisionSolver.AutoStiffness.MassRatio2"), Chaos_PBDCollisionSolver_AutoStiffness_MassRatio2, TEXT(""));

		// Jacobi solver stiffness
		// @todo(chaos): to be tuned
		FRealSingle Chaos_PBDCollisionSolver_JacobiStiffness = 0.5f;
		FAutoConsoleVariableRef CVarChaosPBDCollisionSolverJacobiStiffness(TEXT("p.Chaos.PBDCollisionSolver.JacobiStiffness"), Chaos_PBDCollisionSolver_JacobiStiffness, TEXT(""));

		// Jacobi position tolerance. Position corrections below this are zeroed.
		// @todo(chaos): to be tuned
		FRealSingle Chaos_PBDCollisionSolver_JacobiPositionTolerance = 1.e-6f;
		FAutoConsoleVariableRef CVarChaosPBDCollisionSolverJacobiPositionTolerance(TEXT("p.Chaos.PBDCollisionSolver.JacobiPositionTolerance"), Chaos_PBDCollisionSolver_JacobiPositionTolerance, TEXT(""));

		// Jacobi rotation tolerance. Rotation corrections below this are zeroed.
		// @todo(chaos): to be tuned
		FRealSingle Chaos_PBDCollisionSolver_JacobiRotationTolerance = 1.e-8f;
		FAutoConsoleVariableRef CVarChaosPBDCollisionSolverJacobiRotationTolerance(TEXT("p.Chaos.PBDCollisionSolver.JacobiRotationTolerance"), Chaos_PBDCollisionSolver_JacobiRotationTolerance, TEXT(""));

		// Whether to enable the new initial overlap depentration system
		bool bChaos_Collision_EnableInitialDepenetration = true;
		FAutoConsoleVariableRef CVarChaosCollisionEnableInitialDepentration(TEXT("p.Chaos.PBDCollisionSolver.EnableInitialDepenetration"), bChaos_Collision_EnableInitialDepenetration, TEXT(""));

		// The maximum number of constraints we will attempt to solve (-1 for unlimited)
		int32 Chaos_Collision_MaxSolverManifoldPoints = -1;
		FAutoConsoleVariableRef CVarChaosCollisionMaxSolverManifoldPoints(TEXT("p.Chaos.PBDCollisionSolver.MaxManifoldPoints"), Chaos_Collision_MaxSolverManifoldPoints, TEXT(""));

		// Whether to enable the experimental soft collisions
		bool bChaos_Collision_EnableSoftCollisions = true;
		FAutoConsoleVariableRef CVarChaosCollisionEnableSoftCollisions(TEXT("p.Chaos.PBDCollisionSolver.EnableSoftCollisions"), bChaos_Collision_EnableSoftCollisions, TEXT(""));
	}


	//////////////////////////////////////////////////////////////////////////////////////////////////
	//////////////////////////////////////////////////////////////////////////////////////////////////
	//////////////////////////////////////////////////////////////////////////////////////////////////
	//////////////////////////////////////////////////////////////////////////////////////////////////

	// Transform the Constraint's local-space data into world space for use by  the collision solver and also calculate tangents, errors, etc
	void UpdateCollisionSolverContactPointFromConstraint(
		Private::FPBDCollisionSolver& Solver, 
		const int32 SolverPointIndex, 
		FPBDCollisionConstraint* Constraint, 
		const int32 ConstraintPointIndex, 
		const FRealSingle Dt, 
		const FRealSingle MaxDepenetrationVelocity,
		const FRealSingle MaxPushOut,
		const FConstraintSolverBody& Body0,
		const FConstraintSolverBody& Body1,
		const FPBDCollisionSolverSettings& SolverSettings)
	{
		FManifoldPoint& ManifoldPoint = Constraint->GetManifoldPoint(ConstraintPointIndex);

		const FRealSingle Restitution = FRealSingle(Constraint->GetRestitution());
		const FRealSingle RestitutionVelocityThreshold = FRealSingle(Constraint->GetRestitutionThreshold()) * Dt;

		// World-space shape transforms. Manifold data is currently relative to these spaces
		const FRigidTransform3& ShapeWorldTransform0 = Constraint->GetShapeWorldTransform0();
		const FRigidTransform3& ShapeWorldTransform1 = Constraint->GetShapeWorldTransform1();

		// World-space contact points on each shape
		const FReal ContactWeight0 = Body0.InvM();
		const FReal ContactWeight1 = Body1.InvM();
		const FVec3 WorldContact0 = ShapeWorldTransform0.TransformPositionNoScale(FVec3(ManifoldPoint.ContactPoint.ShapeContactPoints[0]));
		const FVec3 WorldContact1 = ShapeWorldTransform1.TransformPositionNoScale(FVec3(ManifoldPoint.ContactPoint.ShapeContactPoints[1]));
		const FVec3 WorldContact = (ContactWeight0 * WorldContact0 + ContactWeight1 * WorldContact1) / (ContactWeight0 + ContactWeight1);
		const FVec3f WorldRelativeContact0 = FVec3f(WorldContact - Body0.P());
		const FVec3f WorldRelativeContact1 = FVec3f(WorldContact - Body1.P());

		// World-space normal
		const FVec3f WorldContactNormal = FVec3f(ShapeWorldTransform1.TransformVectorNoScale(FVec3(ManifoldPoint.ContactPoint.ShapeContactNormal)));

		// World-space tangent U
		FVec3f WorldContactTangentU = FVec3f::CrossProduct(FVec3f(0, 1, 0), WorldContactNormal);
		if (!WorldContactTangentU.Normalize(FRealSingle(UE_KINDA_SMALL_NUMBER)))
		{
			WorldContactTangentU = FVec3f::CrossProduct(FVec3f(1, 0, 0), WorldContactNormal);
			WorldContactTangentU = WorldContactTangentU.GetUnsafeNormal();
		}

		// Calculate contact velocity if we will need it below (restitution and/or first-contact for friction or initial depenetration)
		const bool bNeedContactVelocity = (!ManifoldPoint.Flags.bHasStaticFrictionAnchor) || (Restitution > FRealSingle(0)) || ManifoldPoint.Flags.bInitialContact || CVars::bChaos_PBDCollisionSolver_Use1DFriction;
		FVec3f ContactVel = FVec3(0);
		FRealSingle ContactVelocityNormal = FRealSingle(0);
		if (bNeedContactVelocity)
		{
			const FVec3f ContactVel0 = Body0.V() + FVec3f::CrossProduct(Body0.W(), WorldRelativeContact0);
			const FVec3f ContactVel1 = Body1.V() + FVec3f::CrossProduct(Body1.W(), WorldRelativeContact1);
			ContactVel = ContactVel0 - ContactVel1;
			ContactVelocityNormal = FVec3f::DotProduct(ContactVel, WorldContactNormal);
			// Align TangentU with the relative sliding velocity direction,
			// Falls back to the geometric axis when sliding velocity is near zero
			FVec3f ContactVelTangent = ContactVel - ContactVelocityNormal * WorldContactNormal;
			if (ContactVelTangent.Normalize(FRealSingle(UE_KINDA_SMALL_NUMBER)))
			{
				WorldContactTangentU = ContactVelTangent;
			}
		}

		// TangentV is always perpendicular to both normal and TangentU
		const FVec3f WorldContactTangentV = FVec3f::CrossProduct(WorldContactNormal, WorldContactTangentU);

		// If we have contact data from a previous tick, use it to calculate the lateral position delta we need
		// to apply to move the contacts back to their original relative locations (i.e., to enforce static friction).
		// Otherwise, estimate the friction correction from the contact velocity.
		// NOTE: quadratic shapes use the velocity-based path most of the time, unless the relative motion is very small.
		FVec3f WorldFrictionDelta = FVec3f(0);
		if (ManifoldPoint.Flags.bHasStaticFrictionAnchor)
		{
			const FVec3 FrictionDelta0 = ShapeWorldTransform0.TransformPositionNoScale(FVec3(ManifoldPoint.ShapeAnchorPoints[0]));
			const FVec3 FrictionDelta1 = ShapeWorldTransform1.TransformPositionNoScale(FVec3(ManifoldPoint.ShapeAnchorPoints[1]));
			WorldFrictionDelta = FVec3f(FrictionDelta0 - FrictionDelta1);
		}
		else
		{
			WorldFrictionDelta = ContactVel * Dt;
		}

		// The contact point error we are trying to correct in this solver
		const FVec3f WorldContactDelta = FVec3f(WorldContact0 - WorldContact1);
		FRealSingle WorldContactDeltaNormal = FVec3f::DotProduct(WorldContactDelta, WorldContactNormal);
		FRealSingle WorldContactDeltaTangentU = FVec3f::DotProduct(WorldContactDelta + WorldFrictionDelta, WorldContactTangentU);
		FRealSingle WorldContactDeltaTangentV = FVec3f::DotProduct(WorldContactDelta + WorldFrictionDelta, WorldContactTangentV);

		// The target contact velocity, taking restitution into account
		FRealSingle WorldContactTargetVelocityNormal = FRealSingle(0);
		if (Restitution > FRealSingle(0))
		{
			if (ContactVelocityNormal < -RestitutionVelocityThreshold)
			{
				WorldContactTargetVelocityNormal = -Restitution * ContactVelocityNormal;
			}
		}

		// Optionally disable friction on the first contact frames
		// @todo(chaos): this should be tied to its own setting rather than piggy-backing on bUseSplitImpulse
		if (ManifoldPoint.Flags.bInitialContact && SolverSettings.bUseSplitImpulse)
		{
			WorldContactDeltaTangentU = 0;
			WorldContactDeltaTangentV = 0;
		}

		// Overlap remaining from the previous frame, estimated from current contact phi and velocity
		FRealSingle WorldContactResidualPhi = FMath::Min(WorldContactDeltaNormal, FRealSingle(0)) - FMath::Min(ContactVelocityNormal * Dt, FRealSingle(0));

		// Handle initial overlaps. Initially overlapping objects will de-penetrate at MaxDepenetrationVelocity (which may be zero).
		if ((MaxDepenetrationVelocity >= 0) && CVars::bChaos_Collision_EnableInitialDepenetration)
		{
			// We can handle initial overlap depenentration individually per manifold point or
			// globally over all maifold points. See FPBDCollisionConstraint::Setup
			const bool bUsePerContactInitialPhi = Constraint->UsePerContactInitialPhi();
			
			// If we have an initial contact, calculate the initial overlap. This will get saved in SetSolverResults
			FRealSingle WorldContactInitialPhi = 0;
			if (ManifoldPoint.Flags.bInitialContact)
			{
				// This is a new manifold point, capture current Phi as the initial Phi
				WorldContactInitialPhi = WorldContactResidualPhi;

				// If this is a new manifold point on a pre-existing manifold, we limit the initial depth.
				// This is so that as we are depenetrating and new points are added to the manifold, we don't suddenly pop out, which would
				// happen if we set InitialPhi=0. But also want to ensure that if we are nearly done handling initial overlap, we don't treat
				// new deeper manifold points as full initial overlaps.
				// NOTE: here we are checking IsInitialContact on the constraint, which is only set when we first make contact, as opposed 
				// to the bInitialContact on the manifold point which is true for any new manifold point, regardless of the constraint age.
				if (bUsePerContactInitialPhi && !Constraint->IsInitialContact())
				{
					WorldContactInitialPhi = FMath::Max(WorldContactInitialPhi, Constraint->GetMinInitialPhi());
				}

			}
			else
			{
				// This is a pre-existing manifold point, but maybe we are still resolving initial penetrations
				// Don't allow this contact to penetrate any deeper than it currently is. InitialPhi will
				// decrease over time if we have a non-zero depenetration velocity
				const FRealSingle InitialPhi = bUsePerContactInitialPhi ? ManifoldPoint.InitialPhi : Constraint->GetMinInitialPhi();
				if (InitialPhi < 0)
				{
					WorldContactInitialPhi = FMath::Max(WorldContactDeltaNormal, InitialPhi);
				}
			}

			// Update the initial overlap based on depenetration velocity
			WorldContactInitialPhi = FMath::Min(WorldContactInitialPhi + MaxDepenetrationVelocity * Dt, 0.0f);

			// Apply initial penetration allowance to depth correction
			WorldContactDeltaNormal -= WorldContactInitialPhi;

			// @todo(chaos): InitialPhi should probably be updated prior to Gather, but this is where we calculate
			// the depth and contact velocity so it's convenient for now. Not great that we have to write back to
			// the constraint here though...
			ManifoldPoint.InitialPhi = WorldContactInitialPhi;
		}

		// Limit the depenetration for this tick if desired
		if (MaxPushOut > 0)
		{
			if (WorldContactDeltaNormal < -MaxPushOut)
			{
				WorldContactDeltaNormal = -MaxPushOut;
			}
		}

		// Adjust depth to account for target penetration from user
		WorldContactDeltaNormal -= ManifoldPoint.TargetPhi;

		FSolverReal SplitImpulseFraction = 0.0f;
		if (SolverSettings.bUseSplitImpulse && !FMath::IsNearlyZero(WorldContactDeltaNormal))
		{
			// SplitImpulseFraction: how much of WorldContactDeltaNormal comes from residual overlap from the previous frame.
			// Use a momentumless correction for all new overlap, but still add momentum to correct any redisual overlap
			SplitImpulseFraction = FMath::Clamp(1.0f - (WorldContactResidualPhi / WorldContactDeltaNormal), 0.0f, 1.0f);
		}

		Solver.InitManifoldPoint(
			SolverPointIndex,
			Dt,
			WorldRelativeContact0,
			WorldRelativeContact1,
			WorldContactNormal,
			WorldContactTangentU,
			WorldContactTangentV,
			WorldContactDeltaNormal,
			WorldContactDeltaTangentU,
			WorldContactDeltaTangentV,
			WorldContactTargetVelocityNormal,
			SplitImpulseFraction,
			ManifoldPoint.Flags.bDisablePositionSolve,
			ManifoldPoint.Flags.bDisableVelocitySolve,
			ManifoldPoint.Flags.bDisableFriction
			);
	}

	void UpdateCollisionSolverManifoldFromConstraint(
		Private::FPBDCollisionSolver& Solver, 
		FPBDCollisionConstraint* Constraint, 
		const FSolverReal Dt, 
		const int32 ConstraintPointBeginIndex,
		const int32 ConstraintPointEndIndex,
		const FPBDCollisionSolverSettings& SolverSettings)
	{
		const FConstraintSolverBody& Body0 = Solver.SolverBody0();
		const FConstraintSolverBody& Body1 = Solver.SolverBody1();

		// MaxDepenetrationVelocity controls the rate at which initial-overlaps are resolved
		const FSolverReal MaxDepenetrationVelocity = Constraint->GetInitialOverlapDepenetrationVelocity();

		// The maximum correction we can apply in one frame
		// @todo(chaos): consider removing this functionality?
		const FSolverReal MaxPushOut = (SolverSettings.MaxPushOutVelocity > 0) ? (FSolverReal(SolverSettings.MaxPushOutVelocity) * Dt) : FSolverReal(0);

		// Only calculate state for newly added contacts. Normally this is all of them, but maybe not if incremental collision is used by RBAN.
		// Also we only add active points to the solver's manifold points list
		for (int32 ConstraintManifoldPointIndex = ConstraintPointBeginIndex; ConstraintManifoldPointIndex < ConstraintPointEndIndex; ++ConstraintManifoldPointIndex)
		{
			if (!Constraint->GetManifoldPoint(ConstraintManifoldPointIndex).Flags.bDisabled)
			{
				const int32 SolverManifoldPointIndex = Solver.AddManifoldPoint();
				if (SolverManifoldPointIndex != INDEX_NONE)
				{
					// Transform the constraint contact data into world space for use by the solver
					// We build this data directly into the solver's world-space contact data which looks a bit odd with "Init" called after but there you go
					UpdateCollisionSolverContactPointFromConstraint(
						Solver, SolverManifoldPointIndex, 
						Constraint, ConstraintManifoldPointIndex, 
						Dt, MaxDepenetrationVelocity, MaxPushOut, 
						Body0, Body1, 
						SolverSettings);
				}
			}
		}

		Solver.FinalizeManifold();
	}

	void UpdateCollisionSolverFromConstraint(
		Private::FPBDCollisionSolver& Solver, 
		FPBDCollisionConstraint* Constraint, 
		const FSolverReal Dt, 
		const FPBDCollisionSolverSettings& SolverSettings, 
		bool& bOutPerIterationCollision)
	{
		// Friction values. Static and Dynamic friction are applied in the position solve for most shapes.
		// We can also run in a mode without static friction at all. This is faster but stacking is not possible.
		const FSolverReal StaticFriction = FSolverReal(Constraint->GetStaticFriction());
		const FSolverReal DynamicFriction = FSolverReal(Constraint->GetDynamicFriction());
		FSolverReal PositionStaticFriction = FSolverReal(0);
		FSolverReal PositionDynamicFriction = FSolverReal(0);
		FSolverReal VelocityDynamicFriction = FSolverReal(0);
		FSolverReal MinFrictionPushOut = FSolverReal(Constraint->GetMinFrictionPushOut());
		if (SolverSettings.NumPositionFrictionIterations > 0)
		{
			PositionStaticFriction = StaticFriction;
			PositionDynamicFriction = DynamicFriction;
		}
		if (SolverSettings.NumVelocityFrictionIterations > 0)
		{
			VelocityDynamicFriction = DynamicFriction;
		}

		Solver.SetFriction(PositionStaticFriction, PositionDynamicFriction, VelocityDynamicFriction, MinFrictionPushOut);


		const FReal SolverStiffness = Constraint->GetStiffness();

		Solver.SetStiffness(FSolverReal(SolverStiffness));

		Solver.SetHardContact();

		Solver.SolverBody0().SetInvMScale(Constraint->GetInvMassScale0());
		Solver.SolverBody0().SetInvIScale(Constraint->GetInvInertiaScale0());
		Solver.SolverBody0().SetShockPropagationScale(FReal(1));
		Solver.SolverBody1().SetInvMScale(Constraint->GetInvMassScale1());
		Solver.SolverBody1().SetInvIScale(Constraint->GetInvInertiaScale1());
		Solver.SolverBody1().SetShockPropagationScale(FReal(1));

		bOutPerIterationCollision = (!Constraint->GetUseManifold() || Constraint->GetUseIncrementalCollisionDetection());

		if (Solver.SolverBody0().IsDynamic() ||
			Solver.SolverBody1().IsDynamic())
		{
			UpdateCollisionSolverManifoldFromConstraint(Solver, Constraint, Dt, 0, Constraint->NumManifoldPoints(), SolverSettings);
		}

		// Convert to a soft collision if the constraint has a nonzero soft separation
		// NOTE: must come after UpdateCollisionSolverManifoldFromConstraint because we scale the spring forces by the number of manifold points.
		if (Constraint->IsSoftContact() && CVars::bChaos_Collision_EnableSoftCollisions)
		{
			// NOTE: convert stiffness and damping into XPBD terms
			if (Solver.NumManifoldPoints() > 0)
			{
				const FSolverReal SoftScale = FSolverReal(1) / FSolverReal(Solver.NumManifoldPoints());
				Solver.SetSoftContact(Constraint->GetSoftSeparation());
			}
		}
	}

	FORCEINLINE_DEBUGGABLE void UpdateCollisionConstraintFromSolver(FPBDCollisionConstraint* Constraint, const Private::FPBDCollisionSolver& Solver, const FSolverReal Dt)
	{
		Constraint->ResetSolverResults();

		// NOTE: We only put the non-pruned manifold points into the solver so the ManifoldPointIndex and
		// SolverManifoldPointIndex do not necessarily match. See GatherManifoldPoints
		int32 SolverManifoldPointIndex = 0;
		for (int32 ManifoldPointIndex = 0; ManifoldPointIndex < Constraint->NumManifoldPoints(); ++ManifoldPointIndex)
		{
			FSolverVec3 NetPushOut = FSolverVec3(0);
			FSolverVec3 NetImpulse = FSolverVec3(0);
			FSolverReal StaticFrictionRatio = FSolverReal(0);

			if (!Constraint->GetManifoldPoint(ManifoldPointIndex).Flags.bDisabled)
			{
				if (SolverManifoldPointIndex < Solver.NumManifoldPoints())
				{
					const Private::FPBDCollisionSolverManifoldPoint& SolverManifoldPoint = Solver.GetManifoldPoint(SolverManifoldPointIndex);

					NetPushOut =
						SolverManifoldPoint.NetPushOutNormal * SolverManifoldPoint.ContactNormal +
						SolverManifoldPoint.NetPushOutTangentU * SolverManifoldPoint.ContactTangentU +
						SolverManifoldPoint.NetPushOutTangentV * SolverManifoldPoint.ContactTangentV;

					NetImpulse =
						SolverManifoldPoint.NetImpulseNormal * SolverManifoldPoint.ContactNormal +
						SolverManifoldPoint.NetImpulseTangentU * SolverManifoldPoint.ContactTangentU +
						SolverManifoldPoint.NetImpulseTangentV * SolverManifoldPoint.ContactTangentV;

					StaticFrictionRatio = SolverManifoldPoint.StaticFrictionRatio;

					++SolverManifoldPointIndex;
				}
			}

			// NOTE: We call this even for points we did not run the solver for (but with zero results)
			Constraint->SetSolverResults(ManifoldPointIndex,
				NetPushOut,
				NetImpulse,
				StaticFrictionRatio,
				Dt);
		}
	}

	//////////////////////////////////////////////////////////////////////////////////////////////////
	//////////////////////////////////////////////////////////////////////////////////////////////////
	//////////////////////////////////////////////////////////////////////////////////////////////////
	//////////////////////////////////////////////////////////////////////////////////////////////////


	FPBDCollisionContainerSolver::FPBDCollisionContainerSolver(const FPBDCollisionConstraints& InConstraintContainer, const int32 InPriority)
		: FConstraintContainerSolver(InPriority)
		, ConstraintContainer(InConstraintContainer)
		, CollisionConstraints()
		, AppliedShockPropagation()
		, Scratch()
		, CollisionSolvers(nullptr)
		, CollisionSolverManifoldPoints(nullptr)
		, NumCollisionSolverManifoldPoints(0)
		, MaxCollisionSolverManifoldPoints(0)
		, bCollisionConstraintPerIterationCollisionDetection()
		, bPerIterationCollisionDetection(false)
	{
	}

	FPBDCollisionContainerSolver::~FPBDCollisionContainerSolver()
	{
	}

	void FPBDCollisionContainerSolver::ResetImpl(const int32 MaxCollisions)
	{
		const int CollisionBufferNum = CalculateCollisionBufferSize(MaxCollisions, CollisionConstraints.Num());
		CollisionConstraints.Reset(CollisionBufferNum);
		AppliedShockPropagation.Init(1, CollisionBufferNum);
		bCollisionConstraintPerIterationCollisionDetection.Reset(CollisionBufferNum);

		// Just set the array size for these right away - all data will be initialized later
		bCollisionConstraintPerIterationCollisionDetection.SetNumUninitialized(MaxCollisions, EAllowShrinking::No);

		// Reset the solver buffers. We could manifold points as constraints are added,
		// and re-allocate and assign the scratch buffers after
		NumCollisionSolverManifoldPoints = 0;
		MaxCollisionSolverManifoldPoints = 0;
		CollisionSolvers = nullptr;
		CollisionSolverManifoldPoints = nullptr;
	}

	size_t FPBDCollisionContainerSolver::CalculateCollisionBufferSize(const size_t InTightFittingNum, const size_t InCurrentBufferNum) const
	{
		// A buffer over-allocation policy to avoid reallocation every frame in the common case where a pile of objects is dropped
		// and the number of contacts increases every tick. Used for collision solvers and manifold points
		size_t CollisionBufferNum = InTightFittingNum;
		if (CollisionBufferNum > InCurrentBufferNum)
		{
			CollisionBufferNum = (5 * InTightFittingNum) / 4; // +25%
		}
		return CollisionBufferNum;
	}

	int32 FPBDCollisionContainerSolver::CalculateConstraintMaxManifoldPoints(const FPBDCollisionConstraint* Constraint) const
	{
		// NOTE: we don't know how many points we will create if collision detection is deferred or incrememntal, so we just assume 4
		// @todo(chaos): see if we can do better here
		const bool bDeferredCollisionDetection = ConstraintContainer.GetDetectorSettings().bDeferNarrowPhase;
		const int32 ManifoldPointMax = (bDeferredCollisionDetection || Constraint->GetUseIncrementalManifold()) ? 4 : Constraint->NumManifoldPoints();
		return ManifoldPointMax;
	}

	void FPBDCollisionContainerSolver::PrepareSolverBuffer()
	{
		NumCollisionSolverManifoldPoints = 0;
		MaxCollisionSolverManifoldPoints = 0;
		NumCollisionSolvers = 0;
		CollisionSolvers = nullptr;
		CollisionSolverManifoldPoints = nullptr;

		// We have one solver per constraint, unless we have too manu constraints...
		NumCollisionSolvers = CollisionConstraints.Num();

		// Count the manifold points
		// @todo(chaos): can we avoid this?
		const int32 ManifoldPointLimit = CVars::Chaos_Collision_MaxSolverManifoldPoints;
		for (int32 ConstraintIndex = 0; ConstraintIndex < CollisionConstraints.Num(); ++ConstraintIndex)
		{
			const int32 MaxManifoldPoints = CalculateConstraintMaxManifoldPoints(GetConstraint(ConstraintIndex));
			
			// Drop some of the constraints if we exceed some tunable maximum. This is purely to prevent massive 
			// slowdowns or excessive scratch allocations when too many collisions are generated.
			if ((ManifoldPointLimit >= 0) && (MaxCollisionSolverManifoldPoints + MaxManifoldPoints > ManifoldPointLimit))
			{
				// At this point something is assumed to have gone wrong, so this is an error condition.
				UE_LOGF(LogChaos, Error, "FPBDCollisionContainerSolver: exceeded solver manifold point limit %d at constraint %d of %d. This and remaining constraints will not be solved", ManifoldPointLimit, ConstraintIndex, CollisionConstraints.Num());
				NumCollisionSolvers = ConstraintIndex;
				break;
			}

			MaxCollisionSolverManifoldPoints += MaxManifoldPoints;
		}

		// If we have no manifold points, there's no point creating any solvers
		if (MaxCollisionSolverManifoldPoints == 0)
		{
			NumCollisionSolvers = 0;
			return;
		}

		// Resize the scratch buffer (up to 25% slack - see CalculateCollisionBufferGrowSize)
		constexpr size_t AlignedSolverSize = Align(sizeof(Private::FPBDCollisionSolver), alignof(Private::FPBDCollisionSolver));
		constexpr size_t AlignedPointSize = Align(sizeof(Private::FPBDCollisionSolverManifoldPoint), alignof(Private::FPBDCollisionSolverManifoldPoint));
		const size_t ScratchSize = NumCollisionSolvers * AlignedSolverSize + MaxCollisionSolverManifoldPoints * AlignedPointSize;
		const size_t ScratchBufferSize = CalculateCollisionBufferSize(ScratchSize, Scratch.BufferSize());
		Scratch.Reset(ScratchBufferSize);
			
		// Out of memory?
		if (Scratch.BufferSize() == 0)
		{
			UE_LOGF(LogChaos, Error, "FPBDCollisionContainerSolver: failed to allocate scratch buffer of size %lld bytes. NumCollisions=%d, NumManifoldPoints=%d. Collisions will be lost.", ScratchBufferSize, NumCollisionSolvers, MaxCollisionSolverManifoldPoints);
			MaxCollisionSolverManifoldPoints = 0;
			NumCollisionSolvers = 0;
			return;
		}

		// Allocate scratch space for the collision solvers and manifold points
		CollisionSolvers = Scratch.AllocArray<Private::FPBDCollisionSolver>(NumCollisionSolvers);
		CollisionSolverManifoldPoints = Scratch.AllocArray<Private::FPBDCollisionSolverManifoldPoint>(MaxCollisionSolverManifoldPoints);

		// We should never see these errors if the size calculations are correct, but being extra careful...
		if ((CollisionSolvers == nullptr) || (CollisionSolverManifoldPoints == nullptr))
		{
			UE_CLOGF((CollisionSolvers == nullptr), LogChaos, Error, "FPBDCollisionContainerSolver: failed to allocate CollisionSolvers in scratch buffer of size %lld bytes. NumCollisions=%d, NumManifoldPoints=%d. Collisions will be lost.", ScratchBufferSize, NumCollisionSolvers, MaxCollisionSolverManifoldPoints);
			UE_CLOGF((CollisionSolverManifoldPoints == nullptr), LogChaos, Error, "FPBDCollisionContainerSolver: failed to allocate CollisionSolverManifoldPoints in scratch buffer of size %lld bytes. NumCollisions=%d, NumManifoldPoints=%d. Collisions will be lost.", ScratchBufferSize, NumCollisionSolvers, MaxCollisionSolverManifoldPoints);
			MaxCollisionSolverManifoldPoints = 0;
			NumCollisionSolvers = 0;
			CollisionSolvers = nullptr;
			CollisionSolverManifoldPoints = nullptr;
			return;
		}
	}

	void FPBDCollisionContainerSolver::AddConstraints()
	{
		Reset(ConstraintContainer.NumConstraints());

		for (FPBDCollisionConstraintHandle* ConstraintHandle : ConstraintContainer.GetConstraintHandles())
		{
			check(ConstraintHandle != nullptr);
			FPBDCollisionConstraint& Constraint = ConstraintHandle->GetContact();

			AddConstraint(Constraint);
		}
	}

	void FPBDCollisionContainerSolver::AddConstraints(const TArrayView<Private::FPBDIslandConstraint*>& IslandConstraints)
	{
		for (Private::FPBDIslandConstraint* IslandConstraint : IslandConstraints)
		{
			// Validate the constraint sleep state matches between the FPBDIslandConstraint and FConstraintHandle
			ensureMsgf(IslandConstraint->IsSleeping() == IslandConstraint->GetConstraint()->IsSleeping(),
				TEXT("Mismatch in constraint sleep state. Particle 0: %s, Particle 1: %s."),
				*(IslandConstraint->GetConstraint()->GetConstrainedParticles()[0]->GetDebugName()),
				*(IslandConstraint->GetConstraint()->GetConstrainedParticles()[1]->GetDebugName()));

			// Filter out sleeping constraints in any partially sleeping island
			if (!IslandConstraint->IsSleeping())
			{
				FConstraintHandle* Constraint = IslandConstraint->GetConstraint();

				// We will only ever be given constraints from our container (NOTE: AsUnsafe asserts in dev)
				FPBDCollisionConstraint& CollisionConstraint = Constraint->AsUnsafe<FPBDCollisionConstraintHandle>()->GetContact();

				AddConstraint(CollisionConstraint);
			}
		}
	}

	void FPBDCollisionContainerSolver::AddConstraint(FPBDCollisionConstraint& Constraint)
	{
		// NOTE: No need to add to CollisionSolvers or bCollisionConstraintPerIterationCollisionDetection - handled later
		CollisionConstraints.Add(&Constraint);
	}

	bool FPBDCollisionContainerSolver::SortConstraintsByPartitionIndex(const TArray<int32>& PartitionIndices, const TArray<int32>& PrefixSum)
	{
		Private::SortArrayByPartitionIndex(CollisionConstraints, PartitionIndices, PrefixSum);
		return true;
	}

	void FPBDCollisionContainerSolver::GetAllConstrainedParticles(TArray<const FGeometryParticleHandle*>& ParticleHandles) const
	{
		ParticleHandles.Reset();
		ParticleHandles.Reserve(CollisionConstraints.Num() * 2);

		for (const FPBDCollisionConstraint* Constraint : CollisionConstraints)
		{
			ParticleHandles.Add(Constraint->GetParticle0());
			ParticleHandles.Add(Constraint->GetParticle1());
		}
	}

	void FPBDCollisionContainerSolver::AddBodies(FSolverBodyContainer& SolverBodyContainer)
	{
		// All constraints are now added. We can allocate the solver buffers.
		PrepareSolverBuffer();

		if (CollisionSolverManifoldPoints == nullptr)
		{
			return;
		}

		for (int32 ConstraintIndex = 0, ConstraintEndIndex = NumSolvers(); ConstraintIndex < ConstraintEndIndex; ++ConstraintIndex)
		{
			Private::FPBDCollisionSolver& CollisionSolver = GetSolver(ConstraintIndex);
			FPBDCollisionConstraint* Constraint = GetConstraint(ConstraintIndex);
			check(Constraint != nullptr);

			// Find the solver bodies for the particles we constrain. This will add them to the container
			// if they aren't there already, and ensure that they are populated with the latest data.
			FSolverBody* Body0 = SolverBodyContainer.FindOrAdd(Constraint->GetParticle0());
			FSolverBody* Body1 = SolverBodyContainer.FindOrAdd(Constraint->GetParticle1());
			check(Body0 != nullptr);
			check(Body1 != nullptr);

			// Set up the solver manifold point buffer pointer
			const int32 ConstraintManifoldPointMax = CalculateConstraintMaxManifoldPoints(GetConstraint(ConstraintIndex));

			// We should have allocated enough space for all manifold points in PrepareSolverBuffer
			check(NumCollisionSolverManifoldPoints + ConstraintManifoldPointMax <= MaxCollisionSolverManifoldPoints);

			CollisionSolver.Reset(&CollisionSolverManifoldPoints[NumCollisionSolverManifoldPoints], ConstraintManifoldPointMax);
			CollisionSolver.SetSolverBodies(*Body0, *Body1);

			NumCollisionSolverManifoldPoints += ConstraintManifoldPointMax;
		}
	}

	void FPBDCollisionContainerSolver::GatherInput(const FReal Dt)
	{
		GatherInput(Dt, 0, NumSolvers());
	}

	void FPBDCollisionContainerSolver::CachePrefetchSolver(const int32 ConstraintIndex) const
	{
		if (ConstraintIndex < NumSolvers())
		{
			FPlatformMisc::PrefetchBlock(GetConstraint(ConstraintIndex), sizeof(FPBDCollisionConstraint));
			FPlatformMisc::PrefetchBlock(&GetSolver(ConstraintIndex), sizeof(Private::FPBDCollisionSolver));
		}
	}

	void FPBDCollisionContainerSolver::GatherInput(const FReal InDt, const int32 ConstraintBeginIndex, const int32 ConstraintEndIndex)
	{
		// NOTE: may be called in parallel. Should not change the container or any elements outside of [BeginIndex, EndIndex)

		check(ConstraintBeginIndex >= 0);
		check(ConstraintEndIndex <= CollisionConstraints.Num());

		// Handle the case where we dropped some constraints because there were too  many
		const int32 BeginIndex = FMath::Min(ConstraintBeginIndex, NumCollisionSolvers);
		const int32 EndIndex = FMath::Min(ConstraintEndIndex, NumCollisionSolvers);

		const FSolverReal Dt = FSolverReal(InDt);

		const int32 PrefetchCount = 2;
		for (int32 PrefetchIndex = 0; PrefetchIndex < PrefetchCount; ++PrefetchIndex)
		{
			CachePrefetchSolver(PrefetchIndex);
		}

		bool bWantPerIterationCollisionDetection = false;
		for (int32 ConstraintIndex = BeginIndex; ConstraintIndex < EndIndex; ++ConstraintIndex)
		{
			CachePrefetchSolver(ConstraintIndex + PrefetchCount);

			Private::FPBDCollisionSolver& CollisionSolver = GetSolver(ConstraintIndex);
			FPBDCollisionConstraint* Constraint = GetConstraint(ConstraintIndex);
			bool& bPerIterationCollision = bCollisionConstraintPerIterationCollisionDetection[ConstraintIndex];

			UpdateCollisionSolverFromConstraint(CollisionSolver, Constraint, Dt, ConstraintContainer.GetSolverSettings(), bPerIterationCollision);

			// We need to run collision every iteration if we are not using manifolds, or are using incremental manifolds
			bWantPerIterationCollisionDetection = bWantPerIterationCollisionDetection || bPerIterationCollision;
		}

		if (bWantPerIterationCollisionDetection)
		{
			// We should lock here? We only ever set to true or do nothing so I think it doesn't matter if this happens on multiple threads...
			bPerIterationCollisionDetection = true;
		}
	}

	void FPBDCollisionContainerSolver::ScatterOutput(const FReal Dt)
	{
		ScatterOutput(Dt, 0, NumSolvers());
	}

	void FPBDCollisionContainerSolver::ScatterOutput(const FReal InDt, const int32 ConstraintBeginIndex, const int32 ConstraintEndIndex)
	{
		// NOTE: may be called in parallel. Should not change the container or any elements outside of [BeginIndex, EndIndex)

		check(ConstraintBeginIndex >= 0);
		check(ConstraintEndIndex <= CollisionConstraints.Num());

		// Handle the case where we dropped some constraints because there were too  many
		const int32 BeginIndex = FMath::Min(ConstraintBeginIndex, NumCollisionSolvers);
		const int32 EndIndex = FMath::Min(ConstraintEndIndex, NumCollisionSolvers);

		const FSolverReal Dt = FSolverReal(InDt);

		for (int32 ConstraintIndex = BeginIndex; ConstraintIndex < EndIndex; ++ConstraintIndex)
		{
			Private::FPBDCollisionSolver& CollisionSolver = GetSolver(ConstraintIndex);
			FPBDCollisionConstraint* Constraint = GetConstraint(ConstraintIndex);

			UpdateCollisionConstraintFromSolver(Constraint, CollisionSolver, Dt);

			// Reset the collision solver here as the pointers will be invalid on the next tick
			// @todo(chaos): Pointers are always reinitialized before use next tick so this isn't strictly necessary
			CollisionSolver.Reset(nullptr, 0);
		}
	}

	void FPBDCollisionContainerSolver::ApplyPositionConstraints(const FReal Dt, const int32 It, const int32 NumIts)
	{
		check(NumSolvers() <= GetNumConstraints());
		ApplyPositionConstraints(Dt, It, NumIts, 0, NumSolvers());
	}

	void FPBDCollisionContainerSolver::ApplyPositionConstraints(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex)
	{
		check(NumSolvers() <= GetNumConstraints());
		SolvePositionImpl(Dt, It, NumIts, BeginIndex, EndIndex, ConstraintContainer.GetSolverSettings());
	}

	void FPBDCollisionContainerSolver::ApplyVelocityConstraints(const FReal Dt, const int32 It, const int32 NumIts)
	{
		check(NumSolvers() <= GetNumConstraints());
		ApplyVelocityConstraints(Dt, It, NumIts, 0, NumSolvers());
	}

	void FPBDCollisionContainerSolver::ApplyVelocityConstraints(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex)
	{
		check(NumSolvers() <= GetNumConstraints());
		SolveVelocityImpl(Dt, It, NumIts, BeginIndex, EndIndex, ConstraintContainer.GetSolverSettings());
	}

	void FPBDCollisionContainerSolver::ApplyProjectionConstraints(const FReal Dt, const int32 It, const int32 NumIts)
	{
		check(NumSolvers() <= GetNumConstraints());
		ApplyProjectionConstraints(Dt, It, NumIts, 0, NumSolvers());
	}

	void FPBDCollisionContainerSolver::ApplyProjectionConstraints(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex)
	{
		// Not supported for collisions
	}

	void FPBDCollisionContainerSolver::ApplyShockPropagation(const FSolverReal ShockPropagation, const int32 BeginIndex, const int32 EndIndex)
	{
		// @todo(chaos): cache the mass scales so we don't have to look in the constraint again
		for (int32 SolverIndex = BeginIndex; SolverIndex < EndIndex; ++SolverIndex)
		{
			if (ShockPropagation != AppliedShockPropagation[SolverIndex])
			{
				const FPBDCollisionConstraint* Constraint = GetConstraint(SolverIndex);
				Private::FPBDCollisionSolver& Solver = GetSolver(SolverIndex);
				if (Constraint != nullptr)
				{
					FConstraintSolverBody& Body0 = Solver.SolverBody0();
					FConstraintSolverBody& Body1 = Solver.SolverBody1();

					FSolverReal ShockPropagation0, ShockPropagation1;
					if (Private::CalculateBodyShockPropagation(Body0.SolverBody(), Body1.SolverBody(), ShockPropagation, ShockPropagation0, ShockPropagation1))
					{
						Body0.SetShockPropagationScale(ShockPropagation0);
						Body1.SetShockPropagationScale(ShockPropagation1);
						Solver.UpdateMassNormal();
					}
				}

				AppliedShockPropagation[SolverIndex] = ShockPropagation;
			}
		}
	}

	void FPBDCollisionContainerSolver::UpdatePositionShockPropagation(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex, const FPBDCollisionSolverSettings& SolverSettings)
	{
		const bool bEnableShockPropagation = (It >= NumIts - SolverSettings.NumPositionShockPropagationIterations);
		const FSolverReal ShockPropagation = (bEnableShockPropagation) ? CVars::Chaos_PBDCollisionSolver_Position_MinInvMassScale : FSolverReal(1);
		ApplyShockPropagation(ShockPropagation, BeginIndex, EndIndex);
	}

	void FPBDCollisionContainerSolver::UpdateVelocityShockPropagation(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex, const FPBDCollisionSolverSettings& SolverSettings)
	{
		const bool bEnableShockPropagation = (It >= NumIts - SolverSettings.NumVelocityShockPropagationIterations);
		const FSolverReal ShockPropagation = (bEnableShockPropagation) ? CVars::Chaos_PBDCollisionSolver_Velocity_MinInvMassScale : FSolverReal(1);
		ApplyShockPropagation(ShockPropagation, BeginIndex, EndIndex);
	}

	void FPBDCollisionContainerSolver::SolvePositionImpl(const FReal InDt, const int32 It, const int32 NumIts, const int32 ConstraintBeginIndex, const int32 ConstraintEndIndex, const FPBDCollisionSolverSettings& SolverSettings)
	{
		SCOPE_CYCLE_COUNTER(STAT_Collisions_Apply);
		if (!CVars::bChaos_PBDCollisionSolver_Position_SolveEnabled)
		{
			return;
		}

		if (ConstraintEndIndex <= ConstraintBeginIndex)
		{
			return;
		}

		// NOTE: may be called in parallel. Should not change the container or any elements outside of [BeginIndex, EndIndex)
		check(ConstraintBeginIndex >= 0);
		check(ConstraintEndIndex <= CollisionConstraints.Num());

		// Handle the case where we dropped some constraints because there were too  many
		const int32 BeginIndex = FMath::Min(ConstraintBeginIndex, NumCollisionSolvers);
		const int32 EndIndex = FMath::Min(ConstraintEndIndex, NumCollisionSolvers);

		if (EndIndex <= BeginIndex)
		{
			return;
		}

		UpdatePositionShockPropagation(InDt, It, NumIts, BeginIndex, EndIndex, SolverSettings);

		// We run collision detection here under two conditions (normally it is run after Integration and before the constraint solver phase):
		// 1) When deferring collision detection until the solver phase for better joint-collision behaviour (RBAN). In this case, we only do this on the first iteration.
		// 2) When using no manifolds or incremental manifolds, where we may add/replace manifold points every iteration.
		const bool bRunDeferredCollisionDetection = (It == 0) && ConstraintContainer.GetDetectorSettings().bDeferNarrowPhase;
		if (bRunDeferredCollisionDetection || bPerIterationCollisionDetection)
		{
			UpdateCollisions(InDt, BeginIndex, EndIndex);
		}

		// Only apply friction for the last few (tunable) iterations
		// Adjust max pushout to attempt to make it iteration count independent
		const FSolverReal Dt = FSolverReal(InDt);
		const bool bApplyStaticFriction = (It >= (NumIts - SolverSettings.NumPositionFrictionIterations));
		const FSolverReal MaxPushOut = FSolverReal(0);	// Now handled in UpdateCollisionSolverContactPointFromConstraint

		// Apply the position correction
		if (bApplyStaticFriction)
		{
			for (int32 SolverIndex = BeginIndex; SolverIndex < EndIndex; ++SolverIndex)
			{
				CollisionSolvers[SolverIndex].SolvePositionWithFriction(Dt, MaxPushOut);
			}
		}
		else
		{
			for (int32 SolverIndex = BeginIndex; SolverIndex < EndIndex; ++SolverIndex)
			{
				CollisionSolvers[SolverIndex].SolvePositionNoFriction(Dt, MaxPushOut);
			}
		}
	}

	void FPBDCollisionContainerSolver::SolveVelocityImpl(const FReal InDt, const int32 It, const int32 NumIts, const int32 ConstraintBeginIndex, const int32 ConstraintEndIndex, const FPBDCollisionSolverSettings& SolverSettings)
	{
		SCOPE_CYCLE_COUNTER(STAT_Collisions_ApplyPushOut);
		if (!CVars::bChaos_PBDCollisionSolver_Velocity_SolveEnabled)
		{
			return;
		}

		if (ConstraintEndIndex <= ConstraintBeginIndex)
		{
			return;
		}

		// NOTE: may be called in parallel. Should not change the container or any elements outside of [BeginIndex, EndIndex)
		check(ConstraintBeginIndex >= 0);
		check(ConstraintEndIndex <= CollisionConstraints.Num());

		// Handle the case where we dropped some constraints because there were too  many
		const int32 BeginIndex = FMath::Min(ConstraintBeginIndex, NumCollisionSolvers);
		const int32 EndIndex = FMath::Min(ConstraintEndIndex, NumCollisionSolvers);

		if (EndIndex <= BeginIndex)
		{
			return;
		}

		UpdateVelocityShockPropagation(InDt, It, NumIts, BeginIndex, EndIndex, SolverSettings);

		const FSolverReal Dt = FSolverReal(InDt);
		const bool bApplyDynamicFriction = (It >= NumIts - SolverSettings.NumVelocityFrictionIterations);

		for (int32 SolverIndex = BeginIndex; SolverIndex < EndIndex; ++SolverIndex)
		{
			CollisionSolvers[SolverIndex].SolveVelocity(Dt, bApplyDynamicFriction);
		}
	}

	void FPBDCollisionContainerSolver::UpdateCollisions(const FReal InDt, const int32 BeginIndex, const int32 EndIndex)
	{
		const FSolverReal Dt = FSolverReal(InDt);
		const bool bDeferredCollisionDetection = ConstraintContainer.GetDetectorSettings().bDeferNarrowPhase;

		bool bNeedsAnotherIteration = false;
		for (int32 SolverIndex = BeginIndex; SolverIndex < EndIndex; ++SolverIndex)
		{
			if (bDeferredCollisionDetection || bCollisionConstraintPerIterationCollisionDetection[SolverIndex])
			{
				Private::FPBDCollisionSolver& CollisionSolver = GetSolver(SolverIndex);
				FPBDCollisionConstraint* Constraint = GetConstraint(SolverIndex);

				// Run collision detection at the current transforms including any correction from previous iterations
				const FConstraintSolverBody& Body0 = CollisionSolver.SolverBody0();
				const FConstraintSolverBody& Body1 = CollisionSolver.SolverBody1();
				const FRigidTransform3 CorrectedActorWorldTransform0 = FRigidTransform3(Body0.CorrectedActorP(), Body0.CorrectedActorQ());
				const FRigidTransform3 CorrectedActorWorldTransform1 = FRigidTransform3(Body1.CorrectedActorP(), Body1.CorrectedActorQ());
				const FRigidTransform3 CorrectedShapeWorldTransform0 = Constraint->GetShapeRelativeTransform0() * CorrectedActorWorldTransform0;
				const FRigidTransform3 CorrectedShapeWorldTransform1 = Constraint->GetShapeRelativeTransform1() * CorrectedActorWorldTransform1;

				// @todo(chaos): this is ugly - pass these to the required functions instead and remove from the constraint class
				// This is now only needed for LevelSet collision (see UpdateLevelsetLevelsetConstraint)
				Constraint->SetSolverBodies(&Body0.SolverBody(), &Body1.SolverBody());

				// Reset the manifold if we are not using manifolds (we just use the first manifold point)
				if (!Constraint->GetUseManifold())
				{
					Constraint->ResetActiveManifoldContacts();
					CollisionSolver.ResetManifold();
				}

				// We need to know how many points were added to the manifold
				const int32 BeginPointIndex = Constraint->NumManifoldPoints();

				// NOTE: We deliberately have not updated the ShapwWorldTranforms on the constraint. If we did that, we would calculate 
				// errors incorrectly in UpdateManifoldPoints, because the solver assumes nothing has been moved as we iterate (we accumulate 
				// corrections that will be applied later.)
				Constraint->ResetPhi(Constraint->GetCullDistance());
				Collisions::UpdateConstraint(*Constraint, CorrectedShapeWorldTransform0, CorrectedShapeWorldTransform1, Dt);

				// Update the manifold based on the new or updated contacts
				UpdateCollisionSolverManifoldFromConstraint(
					CollisionSolver, Constraint, 
					Dt,
					BeginPointIndex, Constraint->NumManifoldPoints(),
					ConstraintContainer.GetSolverSettings());

				Constraint->SetSolverBodies(nullptr, nullptr);
			}
		}
	}

}
