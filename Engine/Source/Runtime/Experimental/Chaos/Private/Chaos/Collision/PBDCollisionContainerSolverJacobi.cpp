// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/Collision/PBDCollisionContainerSolverJacobi.h"

#include "Chaos/Collision/CollisionApplyType.h"
#include "Chaos/Collision/PBDCollisionConstraint.h"
#include "Chaos/CollisionResolution.h"
#include "Chaos/DebugDrawQueue.h"
#include "Chaos/Evolution/SolverBodyContainer.h"
#include "Chaos/Island/IslandManager.h"
#include "Chaos/Particle/ParticleUtilities.h"
#include "Chaos/PBDCollisionConstraints.h"
#include "Chaos/PBDCollisionConstraintsContact.h"
#include "Chaos/Utilities.h"

// Private includes

#include "ChaosLog.h"

//PRAGMA_DISABLE_OPTIMIZATION

namespace Chaos
{
	namespace CVars
	{
		extern bool bChaos_PBDCollisionSolver_Position_SolveEnabled;
		extern bool bChaos_PBDCollisionSolver_Velocity_SolveEnabled;

		extern FRealSingle Chaos_PBDCollisionSolver_AutoStiffness_MassRatio1;
		extern FRealSingle Chaos_PBDCollisionSolver_AutoStiffness_MassRatio2;

		extern FRealSingle Chaos_PBDCollisionSolver_JacobiStiffness;
	}

	namespace Private
	{

		//////////////////////////////////////////////////////////////////////////////////////////////////
		//////////////////////////////////////////////////////////////////////////////////////////////////
		//////////////////////////////////////////////////////////////////////////////////////////////////
		//////////////////////////////////////////////////////////////////////////////////////////////////

		// Transform the Constraint's local-space data into world space for use by  the collision solver and also calculate tangents, errors, etc
		FORCEINLINE_DEBUGGABLE void UpdateCollisionSolverContactPointFromConstraint(
			Private::FPBDCollisionSolverJacobi& Solver, 
			const int32 SolverPointIndex, 
			const FPBDCollisionConstraint* Constraint, 
			const int32 ConstraintPointIndex, 
			const FRealSingle Dt, 
			const FSolverBody& Body0, 
			const FSolverBody& Body1)
		{
			const FManifoldPoint& ManifoldPoint = Constraint->GetManifoldPoint(ConstraintPointIndex);

			const FRealSingle Restitution = FRealSingle(Constraint->GetRestitution());
			const FRealSingle RestitutionVelocityThreshold = FRealSingle(Constraint->GetRestitutionThreshold()) * Dt;

			// World-space shape transforms. Manifold data is currently relative to these spaces
			const FRigidTransform3& ShapeWorldTransform0 = Constraint->GetShapeWorldTransform0();
			const FRigidTransform3& ShapeWorldTransform1 = Constraint->GetShapeWorldTransform1();

			// World-space contact points on each shape
			const FVec3 WorldContact0 = ShapeWorldTransform0.TransformPositionNoScale(FVec3(ManifoldPoint.ContactPoint.ShapeContactPoints[0]));
			const FVec3 WorldContact1 = ShapeWorldTransform1.TransformPositionNoScale(FVec3(ManifoldPoint.ContactPoint.ShapeContactPoints[1]));
			const FVec3 WorldContact = FReal(0.5) * (WorldContact0 + WorldContact1);
			const FVec3f WorldRelativeContact0 = FVec3f(WorldContact - Body0.P());
			const FVec3f WorldRelativeContact1 = FVec3f(WorldContact - Body1.P());

			// World-space normal
			const FVec3f WorldContactNormal = FVec3f(ShapeWorldTransform1.TransformVectorNoScale(FVec3(ManifoldPoint.ContactPoint.ShapeContactNormal)));

			// World-space tangents
			FVec3f WorldContactTangentU = FVec3f::CrossProduct(FVec3f(0, 1, 0), WorldContactNormal);
			if (!WorldContactTangentU.Normalize(FRealSingle(UE_KINDA_SMALL_NUMBER)))
			{
				WorldContactTangentU = FVec3f::CrossProduct(FVec3f(1, 0, 0), WorldContactNormal);
				WorldContactTangentU = WorldContactTangentU.GetUnsafeNormal();
			}
			const FVec3f WorldContactTangentV = FVec3f::CrossProduct(WorldContactNormal, WorldContactTangentU);

			// Calculate contact velocity if we will need it below (restitution and/or frist-contact friction)
			const bool bNeedContactVelocity = (!ManifoldPoint.Flags.bHasStaticFrictionAnchor) || (Restitution > FRealSingle(0));
			FVec3f ContactVel = FVec3(0);
			if (bNeedContactVelocity)
			{
				const FVec3f ContactVel0 = Body0.V() + FVec3f::CrossProduct(Body0.W(), WorldRelativeContact0);
				const FVec3f ContactVel1 = Body1.V() + FVec3f::CrossProduct(Body1.W(), WorldRelativeContact1);
				ContactVel = ContactVel0 - ContactVel1;
			}

			// If we have contact data from a previous tick, use it to calculate the lateral position delta we need
			// to apply to move the contacts back to their original relative locations (i.e., to enforce static friction).
			// Otherwise, estimate the friction correction from the contact velocity.
			// NOTE: quadratic shapes use the velocity-based path most of the time, unless the relative motion is very small.
			FVec3f WorldFrictionDelta = FVec3f(0);
			if (ManifoldPoint.Flags.bHasStaticFrictionAnchor)
			{
				const FVec3f FrictionDelta0 = ShapeWorldTransform0.TransformPositionNoScale(FVec3(ManifoldPoint.ShapeAnchorPoints[0])) - WorldContact0;
				const FVec3f FrictionDelta1 = ShapeWorldTransform1.TransformPositionNoScale(FVec3(ManifoldPoint.ShapeAnchorPoints[1])) - WorldContact1;
				WorldFrictionDelta = FrictionDelta0 - FrictionDelta1;
			}
			else
			{
				// @todo(chaos): consider adding a multiplier to the initial contact friction
				WorldFrictionDelta = ContactVel * Dt;
			}

			// The contact point error we are trying to correct in this solver
			const FRealSingle TargetPhi = ManifoldPoint.TargetPhi;
			const FVec3f WorldContactDelta = FVec3f(WorldContact0 - WorldContact1);
			const FRealSingle WorldContactDeltaNormal = FVec3f::DotProduct(WorldContactDelta, WorldContactNormal) - TargetPhi;
			const FRealSingle WorldContactDeltaTangentU = FVec3f::DotProduct(WorldContactDelta + WorldFrictionDelta, WorldContactTangentU);
			const FRealSingle WorldContactDeltaTangentV = FVec3f::DotProduct(WorldContactDelta + WorldFrictionDelta, WorldContactTangentV);

			// The target contact velocity, taking restitution into account
			FRealSingle WorldContactTargetVelocityNormal = FRealSingle(0);
			if (Restitution > FRealSingle(0))
			{
				const FRealSingle ContactVelocityNormal = FVec3f::DotProduct(ContactVel, WorldContactNormal);
				if (ContactVelocityNormal < -RestitutionVelocityThreshold)
				{
					WorldContactTargetVelocityNormal = -Restitution * ContactVelocityNormal;
				}
			}

			Solver.InitManifoldPoint(
				SolverPointIndex,
				WorldRelativeContact0,
				WorldRelativeContact1,
				WorldContactNormal,
				WorldContactTangentU,
				WorldContactTangentV,
				WorldContactDeltaNormal,
				WorldContactDeltaTangentU,
				WorldContactDeltaTangentV,
				WorldContactTargetVelocityNormal);
		}

		FORCEINLINE_DEBUGGABLE void UpdateCollisionSolverManifoldFromConstraint(
			Private::FPBDCollisionSolverJacobi& Solver, 
			const FPBDCollisionConstraint* Constraint, 
			const FSolverReal Dt, 
			const int32 ConstraintPointBeginIndex, 
			const int32 ConstraintPointEndIndex)
		{
			const FSolverBody& Body0 = Solver.SolverBody0().SolverBody();
			const FSolverBody& Body1 = Solver.SolverBody1().SolverBody();

			// Only calculate state for newly added contacts. Normally this is all of them, but maybe not if incremental collision is used by RBAN.
			// Also we only add active points to the solver's manifold points list
			for (int32 ConstraintManifoldPointIndex = ConstraintPointBeginIndex; ConstraintManifoldPointIndex < ConstraintPointEndIndex; ++ConstraintManifoldPointIndex)
			{
				if (!Constraint->GetManifoldPoint(ConstraintManifoldPointIndex).Flags.bDisabled)
				{
					const int32 SolverManifoldPointIndex = Solver.AddManifoldPoint();

					// Transform the constraint contact data into world space for use by the solver
					// We build this data directly into the solver's world-space contact data which looks a bit odd with "Init" called after but there you go
					UpdateCollisionSolverContactPointFromConstraint(Solver, SolverManifoldPointIndex, Constraint, ConstraintManifoldPointIndex, Dt, Body0, Body1);

					// Prepare the solver (set constraint-space mass etc)
					Solver.FinalizeManifoldPoint(SolverManifoldPointIndex);
				}
			}
		}

		FORCEINLINE_DEBUGGABLE void UpdateCollisionSolverFromConstraint(
			Private::FPBDCollisionSolverJacobi& Solver, 
			const FPBDCollisionConstraint* Constraint, 
			const FSolverReal Dt, 
			const FPBDCollisionSolverSettings& SolverSettings, 
			bool& bOutPerIterationCollision)
		{
			// Friction values. Static and Dynamic friction are applied in the position solve for most shapes.
			// We can also run in a mode without static friction at all. This is faster but stacking is not possible.
			const FSolverReal StaticFriction = FSolverReal(Constraint->GetStaticFriction());
			const FSolverReal DynamicFriction = FSolverReal(Constraint->GetDynamicFriction());
			FSolverReal PositionStaticFriction = FSolverReal(0);
			FSolverReal PositionDynamicFriction = FSolverReal(0);
			FSolverReal VelocityDynamicFriction = FSolverReal(0);
			if (SolverSettings.NumPositionFrictionIterations > 0)
			{
				PositionStaticFriction = StaticFriction;
				if (!Constraint->HasQuadraticShape())
				{
					PositionDynamicFriction = DynamicFriction;
				}
				else
				{
					// Quadratic shapes don't use PBD dynamic friction - it has issues at slow speeds where the WxR is
					// less than the position tolerance for friction point matching
					// @todo(chaos): fix PBD dynamic friction on quadratic shapes
					VelocityDynamicFriction = DynamicFriction;
				}
			}
			else
			{
				VelocityDynamicFriction = DynamicFriction;
			}

			Solver.SetFriction(PositionStaticFriction, PositionDynamicFriction, VelocityDynamicFriction);

			const FReal SolverStiffness = Constraint->GetStiffness();
			Solver.SetStiffness(FSolverReal(SolverStiffness));

			Solver.SolverBody0().SetInvMScale(Constraint->GetInvMassScale0());
			Solver.SolverBody0().SetInvIScale(Constraint->GetInvInertiaScale0());
			Solver.SolverBody0().SetShockPropagationScale(FReal(1));
			Solver.SolverBody1().SetInvMScale(Constraint->GetInvMassScale1());
			Solver.SolverBody1().SetInvIScale(Constraint->GetInvInertiaScale1());
			Solver.SolverBody1().SetShockPropagationScale(FReal(1));

			bOutPerIterationCollision = (!Constraint->GetUseManifold() || Constraint->GetUseIncrementalCollisionDetection());

			UpdateCollisionSolverManifoldFromConstraint(Solver, Constraint, Dt, 0, Constraint->NumManifoldPoints());
		}

		FORCEINLINE_DEBUGGABLE void UpdateCollisionConstraintFromSolver(
			FPBDCollisionConstraint* Constraint, 
			const Private::FPBDCollisionSolverJacobi& Solver, 
			const FSolverReal Dt)
		{
			Constraint->ResetSolverResults();

			// NOTE: We only put the non-pruned manifold points into the solver so the ManifoldPointIndex and
			// SolverManifoldPointIndex do not necessarily match. See GatherManifoldPoints
			int32 SolverManifoldPointIndex = 0;
			for (int32 ManifoldPointIndex = 0; ManifoldPointIndex < Constraint->NumManifoldPoints(); ++ManifoldPointIndex)
			{
				FSolverVec3 NetPushOut = FSolverVec3(0);
				FSolverVec3 NetImpulse = FSolverVec3(0);
				FSolverReal StaticFrictionRatio = FSolverReal(0);

				if (!Constraint->GetManifoldPoint(ManifoldPointIndex).Flags.bDisabled)
				{
					NetPushOut = Solver.GetNetPushOut(SolverManifoldPointIndex);

					NetImpulse = Solver.GetNetImpulse(SolverManifoldPointIndex);

					StaticFrictionRatio = Solver.GetStaticFrictionRatio(SolverManifoldPointIndex);

					++SolverManifoldPointIndex;
				}

				// NOTE: We call this even for points we did not run the solver for (but with zero results)
				Constraint->SetSolverResults(ManifoldPointIndex,
					NetPushOut,
					NetImpulse,
					StaticFrictionRatio,
					Dt);
			}
		}

		//////////////////////////////////////////////////////////////////////////////////////////////////
		//////////////////////////////////////////////////////////////////////////////////////////////////
		//////////////////////////////////////////////////////////////////////////////////////////////////
		//////////////////////////////////////////////////////////////////////////////////////////////////


		FPBDCollisionContainerSolverJacobi::FPBDCollisionContainerSolverJacobi(const FPBDCollisionConstraints& InConstraintContainer, const int32 InPriority)
			: FConstraintContainerSolver(InPriority)
			, ConstraintContainer(InConstraintContainer)
			, bPerIterationCollisionDetection(false)
		{
	#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST && INTEL_ISPC
			Private::FPBDCollisionSolverJacobiHelper::CheckISPC();
	#endif
		}

		FPBDCollisionContainerSolverJacobi::~FPBDCollisionContainerSolverJacobi()
		{
		}

		void FPBDCollisionContainerSolverJacobi::ResetImpl(const int32 MaxCollisions)
		{
			// A over-allocation policy to avoid reallocation every frame in the common case where a pile of objects is dropped
			// and the number of contacts increases every tick.
			if (MaxCollisions > CollisionSolvers.Max())
			{
				CollisionSolvers.Reset((5 * MaxCollisions) / 4);	// +25%
			}

			CollisionSolvers.Reset(MaxCollisions);
			CollisionConstraints.Reset(CollisionSolvers.Max());
			bCollisionConstraintPerIterationCollisionDetection.Reset(CollisionSolvers.Max());

			// Just set the array size for these right away - all data will be initialized later and there is no constructor
			// We use CollisionConstraints.Num() to track how many constraints we currently have
			CollisionSolvers.SetNumUninitialized(MaxCollisions);
			bCollisionConstraintPerIterationCollisionDetection.SetNumUninitialized(MaxCollisions);
		}

		void FPBDCollisionContainerSolverJacobi::AddConstraints()
		{
			Reset(ConstraintContainer.NumConstraints());

			for (FPBDCollisionConstraintHandle* ConstraintHandle : ConstraintContainer.GetConstraintHandles())
			{
				check(ConstraintHandle != nullptr);
				FPBDCollisionConstraint& Constraint = ConstraintHandle->GetContact();

				AddConstraint(Constraint);
			}
		}

		void FPBDCollisionContainerSolverJacobi::AddConstraints(const TArrayView<Private::FPBDIslandConstraint*>& IslandConstraints)
		{
			for (Private::FPBDIslandConstraint* IslandConstraint : IslandConstraints)
			{
				// Validate the constraint sleep state matches between the FPBDIslandConstraint and FConstraintHandle
				ensureMsgf(IslandConstraint->IsSleeping() == IslandConstraint->GetConstraint()->IsSleeping(),
					TEXT("Mismatch in constraint sleep state. Particle 0: %s, Particle 1: %s."), 
					*(IslandConstraint->GetConstraint()->GetConstrainedParticles()[0]->GetDebugName()),
					*(IslandConstraint->GetConstraint()->GetConstrainedParticles()[1]->GetDebugName()));

				// Filter out sleeping constraints in any partially sleeping island
				if (!IslandConstraint->IsSleeping())
				{
					FConstraintHandle* Constraint = IslandConstraint->GetConstraint();

					// We will only ever be given constraints from our container (asserts in non-shipping)
					FPBDCollisionConstraint& CollisionConstraint = Constraint->AsUnsafe<FPBDCollisionConstraintHandle>()->GetContact();

					AddConstraint(CollisionConstraint);
				}
			}
		}

		void FPBDCollisionContainerSolverJacobi::AddConstraint(FPBDCollisionConstraint& Constraint)
		{
			// NOTE: No need to add to CollisionSolvers or bCollisionConstraintPerIterationCollisionDetection
			const int32 Index = CollisionConstraints.Add(&Constraint);

			CollisionSolvers[Index].Reset();
		}

		bool FPBDCollisionContainerSolverJacobi::SortConstraintsByPartitionIndex(const TArray<int32>& PartitionIndices, const TArray<int32>& PrefixSum)
		{
			Private::SortArrayByPartitionIndex(CollisionConstraints, PartitionIndices, PrefixSum);
			return true;
		}

		void FPBDCollisionContainerSolverJacobi::GetAllConstrainedParticles(TArray<const FGeometryParticleHandle*>& ParticleHandles) const
		{
			ParticleHandles.Reset();
			ParticleHandles.Reserve(CollisionConstraints.Num() * 2);

			for (const FPBDCollisionConstraint* Constraint : CollisionConstraints)
			{
				ParticleHandles.Add(Constraint->GetParticle0());
				ParticleHandles.Add(Constraint->GetParticle1());
			}
		}

		void FPBDCollisionContainerSolverJacobi::AddBodies(FSolverBodyContainer& SolverBodyContainer)
		{
			for (int32 SolverIndex = 0, SolverEndIndex = NumSolvers(); SolverIndex < SolverEndIndex; ++SolverIndex)
			{
				Private::FPBDCollisionSolverJacobi& CollisionSolver = CollisionSolvers[SolverIndex];
				FPBDCollisionConstraint* Constraint = CollisionConstraints[SolverIndex];
				check(Constraint != nullptr);

				// Find the solver bodies for the particles we constrain. This will add them to the container
				// if they aren't there already, and ensure that they are populated with the latest data.
				FSolverBody* Body0 = SolverBodyContainer.FindOrAdd(Constraint->GetParticle0());
				FSolverBody* Body1 = SolverBodyContainer.FindOrAdd(Constraint->GetParticle1());
				check(Body0 != nullptr);
				check(Body1 != nullptr);

				CollisionSolver.SetSolverBodies(*Body0, *Body1);
			}
		}

		void FPBDCollisionContainerSolverJacobi::GatherInput(const FReal Dt)
		{
			GatherInput(Dt, 0, NumSolvers());
		}

		void FPBDCollisionContainerSolverJacobi::CachePrefetchSolver(const int32 ConstraintIndex) const
		{
			if (ConstraintIndex < NumSolvers())
			{
				FPlatformMisc::PrefetchBlock(&CollisionSolvers[ConstraintIndex], sizeof(FPBDCollisionContainerSolverJacobi));
			}
		}

		void FPBDCollisionContainerSolverJacobi::GatherInput(const FReal InDt, const int32 BeginIndex, const int32 EndIndex)
		{
			// NOTE: may be called in parallel. Should not change the container or any elements outside of [BeginIndex, EndIndex)

			check(BeginIndex >= 0);
			check(EndIndex <= NumSolvers());

			const FSolverReal Dt = FSolverReal(InDt);

			const int32 PrefetchCount = 4;
			for (int32 PrefetchIndex = 0; PrefetchIndex < PrefetchCount; ++PrefetchIndex)
			{
				CachePrefetchSolver(PrefetchIndex);
			}

			bool bWantPerIterationCollisionDetection = false;
			for (int32 ConstraintIndex = BeginIndex; ConstraintIndex < EndIndex; ++ConstraintIndex)
			{
				Private::FPBDCollisionSolverJacobi& CollisionSolver = CollisionSolvers[ConstraintIndex];
				FPBDCollisionConstraint* Constraint = CollisionConstraints[ConstraintIndex];
				bool& bPerIterationCollision = bCollisionConstraintPerIterationCollisionDetection[ConstraintIndex];

				UpdateCollisionSolverFromConstraint(CollisionSolver, Constraint, Dt, ConstraintContainer.GetSolverSettings(), bPerIterationCollision);

				// We need to run collision every iteration if we are not using manifolds, or are using incremental manifolds
				bWantPerIterationCollisionDetection = bWantPerIterationCollisionDetection || bPerIterationCollision;

				CachePrefetchSolver(ConstraintIndex + PrefetchCount);
			}

			if (bWantPerIterationCollisionDetection)
			{
				// We should lock here? We only ever set to true or do nothing so I think it doesn't matter if this happens on multiple threads...
				bPerIterationCollisionDetection = true;
			}
		}

		void FPBDCollisionContainerSolverJacobi::ScatterOutput(const FReal Dt)
		{
			ScatterOutput(Dt, 0, NumSolvers());
		}

		void FPBDCollisionContainerSolverJacobi::ScatterOutput(const FReal InDt, const int32 BeginIndex, const int32 EndIndex)
		{
			// NOTE: may be called in parallel. Should not change the container or any elements outside of [BeginIndex, EndIndex)

			check(BeginIndex >= 0);
			check(EndIndex <= NumSolvers());

			const FSolverReal Dt = FSolverReal(InDt);

			for (int32 ConstraintIndex = BeginIndex; ConstraintIndex < EndIndex; ++ConstraintIndex)
			{
				Private::FPBDCollisionSolverJacobi& CollisionSolver = CollisionSolvers[ConstraintIndex];
				FPBDCollisionConstraint* Constraint = CollisionConstraints[ConstraintIndex];

				UpdateCollisionConstraintFromSolver(Constraint, CollisionSolver, Dt);

				// Reset the collision solver here as the pointers will be invalid on the next tick
				// @todo(chaos): Pointers are always reinitialized before use next tick so this isn't strictly necessary
				CollisionSolver.Reset();
			}
		}

		void FPBDCollisionContainerSolverJacobi::ApplyPositionConstraints(const FReal Dt, const int32 It, const int32 NumIts)
		{
			check(NumSolvers() <= GetNumConstraints());
			ApplyPositionConstraints(Dt, It, NumIts, 0, NumSolvers());
		}

		void FPBDCollisionContainerSolverJacobi::ApplyPositionConstraints(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex)
		{
			check(NumSolvers() <= GetNumConstraints());
			SolvePositionImpl(Dt, It, NumIts, BeginIndex, EndIndex, ConstraintContainer.GetSolverSettings());
		}

		void FPBDCollisionContainerSolverJacobi::ApplyVelocityConstraints(const FReal Dt, const int32 It, const int32 NumIts)
		{
			check(NumSolvers() <= GetNumConstraints());
			ApplyVelocityConstraints(Dt, It, NumIts, 0, NumSolvers());
		}

		void FPBDCollisionContainerSolverJacobi::ApplyVelocityConstraints(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex)
		{
			check(NumSolvers() <= GetNumConstraints());
			SolveVelocityImpl(Dt, It, NumIts, BeginIndex, EndIndex, ConstraintContainer.GetSolverSettings());
		}

		void FPBDCollisionContainerSolverJacobi::ApplyProjectionConstraints(const FReal Dt, const int32 It, const int32 NumIts)
		{
			ApplyProjectionConstraints(Dt, It, NumIts, 0, NumSolvers());
		}

		void FPBDCollisionContainerSolverJacobi::ApplyProjectionConstraints(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex)
		{
			// Not supported for collisions
		}

		void FPBDCollisionContainerSolverJacobi::UpdatePositionShockPropagation(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex, const FPBDCollisionSolverSettings& SolverSettings)
		{
			// If this is the first shock propagation iteration, enable it on each solver
			const bool bEnableShockPropagation = (It == NumIts - SolverSettings.NumPositionShockPropagationIterations);
			if (bEnableShockPropagation)
			{
				for (int32 SolverIndex = BeginIndex; SolverIndex < EndIndex; ++SolverIndex)
				{
					CollisionSolvers[SolverIndex].EnablePositionShockPropagation();
				}
			}
		}

		void FPBDCollisionContainerSolverJacobi::UpdateVelocityShockPropagation(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex, const FPBDCollisionSolverSettings& SolverSettings)
		{
			// Set/reset the shock propagation based on current iteration. The position solve may
			// have left the bodies with a mass scale and we want to change or reset it.
			const bool bEnableShockPropagation = (It == NumIts - SolverSettings.NumVelocityShockPropagationIterations);
			if (bEnableShockPropagation)
			{
				for (int32 SolverIndex = BeginIndex; SolverIndex < EndIndex; ++SolverIndex)
				{
					CollisionSolvers[SolverIndex].EnableVelocityShockPropagation();
				}
			}
			else if (It == 0)
			{
				for (int32 SolverIndex = BeginIndex; SolverIndex < EndIndex; ++SolverIndex)
				{
					CollisionSolvers[SolverIndex].DisableShockPropagation();
				}
			}
		}

		bool FPBDCollisionContainerSolverJacobi::SolvePositionImpl(const FReal InDt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex, const FPBDCollisionSolverSettings& SolverSettings)
		{
			SCOPE_CYCLE_COUNTER(STAT_Collisions_Apply);
			if (!CVars::bChaos_PBDCollisionSolver_Position_SolveEnabled)
			{
				return false;
			}

			if (EndIndex <= BeginIndex)
			{
				return false;
			}

			check(NumSolvers() == GetNumConstraints());

			// NOTE: may be called in parallel. Should not change the container or any elements outside of [BeginIndex, EndIndex)
			check(BeginIndex >= 0);
			check(EndIndex <= GetNumConstraints());

			UpdatePositionShockPropagation(InDt, It, NumIts, BeginIndex, EndIndex, SolverSettings);

			// We run collision detection here under two conditions (normally it is run after Integration and before the constraint solver phase):
			// 1) When deferring collision detection until the solver phase for better joint-collision behaviour (RBAN). In this case, we only do this on the first iteration.
			// 2) When using no manifolds or incremental manifolds, where we may add/replace manifold points every iteration.
			const bool bRunDeferredCollisionDetection = (It == 0) && ConstraintContainer.GetDetectorSettings().bDeferNarrowPhase;
			if (bRunDeferredCollisionDetection || bPerIterationCollisionDetection)
			{
				UpdateCollisions(InDt, BeginIndex, EndIndex);
			}

			// Only apply friction for the last few (tunable) iterations
			// Adjust max pushout to attempt to make it iteration count independent
			const FSolverReal Dt = FSolverReal(InDt);
			const bool bApplyStaticFriction = (It >= (NumIts - SolverSettings.NumPositionFrictionIterations));
			const FSolverReal MaxPushOut = (SolverSettings.MaxPushOutVelocity > 0) ? (FSolverReal(SolverSettings.MaxPushOutVelocity) * Dt) / FSolverReal(NumIts) : FSolverReal(0);

			// Apply the position correction
			if (bApplyStaticFriction)
			{
				Private::FPBDCollisionSolverJacobiHelper::SolvePositionWithFriction(MakeArrayView(&CollisionSolvers[BeginIndex], EndIndex - BeginIndex), Dt, MaxPushOut);
			}
			else
			{
				Private::FPBDCollisionSolverJacobiHelper::SolvePositionNoFriction(MakeArrayView(&CollisionSolvers[BeginIndex], EndIndex - BeginIndex), Dt, MaxPushOut);
			}

			return true;
		}

		bool FPBDCollisionContainerSolverJacobi::SolveVelocityImpl(const FReal InDt, const int32 It, const int32 NumIts, const int32 ConstraintBeginIndex, const int32 ConstraintEndIndex, const FPBDCollisionSolverSettings& SolverSettings)
		{
			SCOPE_CYCLE_COUNTER(STAT_Collisions_ApplyPushOut);
			if (!CVars::bChaos_PBDCollisionSolver_Velocity_SolveEnabled)
			{
				return false;
			}

			if (ConstraintEndIndex <= ConstraintBeginIndex)
			{
				return false;
			}

			check(NumSolvers() <= GetNumConstraints());

			// NOTE: may be called in parallel. Should not change the container or any elements outside of [BeginIndex, EndIndex)
			check(ConstraintBeginIndex >= 0);
			check(ConstraintEndIndex <= GetNumConstraints());

			// Handle the case where we dropped some constraints because there were too  many
			const int32 BeginIndex = FMath::Min(ConstraintBeginIndex, NumSolvers());
			const int32 EndIndex = FMath::Min(ConstraintEndIndex, NumSolvers());

			UpdateVelocityShockPropagation(InDt, It, NumIts, BeginIndex, EndIndex, SolverSettings);

			const FSolverReal Dt = FSolverReal(InDt);
			const bool bApplyDynamicFriction = (It >= NumIts - SolverSettings.NumVelocityFrictionIterations);

			Private::FPBDCollisionSolverJacobiHelper::SolveVelocity(MakeArrayView(&CollisionSolvers[BeginIndex], EndIndex - BeginIndex), Dt, bApplyDynamicFriction);

			return true;
		}

		void FPBDCollisionContainerSolverJacobi::UpdateCollisions(const FReal InDt, const int32 BeginIndex, const int32 EndIndex)
		{
			const FSolverReal Dt = FSolverReal(InDt);
			const bool bDeferredCollisionDetection = ConstraintContainer.GetDetectorSettings().bDeferNarrowPhase;

			bool bNeedsAnotherIteration = false;
			for (int32 SolverIndex = BeginIndex; SolverIndex < EndIndex; ++SolverIndex)
			{
				if (bDeferredCollisionDetection || bCollisionConstraintPerIterationCollisionDetection[SolverIndex])
				{
					Private::FPBDCollisionSolverJacobi& CollisionSolver = CollisionSolvers[SolverIndex];
					FPBDCollisionConstraint* Constraint = CollisionConstraints[SolverIndex];

					// Run collision detection at the current transforms including any correction from previous iterations
					const FSolverBody& Body0 = CollisionSolver.SolverBody0().SolverBody();
					const FSolverBody& Body1 = CollisionSolver.SolverBody1().SolverBody();
					const FRigidTransform3 CorrectedActorWorldTransform0 = FRigidTransform3(Body0.CorrectedActorP(), Body0.CorrectedActorQ());
					const FRigidTransform3 CorrectedActorWorldTransform1 = FRigidTransform3(Body1.CorrectedActorP(), Body1.CorrectedActorQ());
					const FRigidTransform3 CorrectedShapeWorldTransform0 = Constraint->GetShapeRelativeTransform0() * CorrectedActorWorldTransform0;
					const FRigidTransform3 CorrectedShapeWorldTransform1 = Constraint->GetShapeRelativeTransform1() * CorrectedActorWorldTransform1;

					// @todo(chaos): this is ugly - pass these to the required functions instead and remove from the constraint class
					// This is now only needed for LevelSet collision (see UpdateLevelsetLevelsetConstraint)
					Constraint->SetSolverBodies(&Body0, &Body1);

					// Reset the manifold if we are not using manifolds (we just use the first manifold point)
					if (!Constraint->GetUseManifold())
					{
						Constraint->ResetActiveManifoldContacts();
						CollisionSolver.ResetManifold();
					}

					// We need to know how many points were added to the manifold
					const int32 BeginPointIndex = Constraint->NumManifoldPoints();

					// NOTE: We deliberately have not updated the ShapwWorldTranforms on the constraint. If we did that, we would calculate 
					// errors incorrectly in UpdateManifoldPoints, because the solver assumes nothing has been moved as we iterate (we accumulate 
					// corrections that will be applied later.)
					Constraint->ResetPhi(Constraint->GetCullDistance());
					Collisions::UpdateConstraint(*Constraint, CorrectedShapeWorldTransform0, CorrectedShapeWorldTransform1, Dt);

					// Update the manifold based on the new or updated contacts
					UpdateCollisionSolverManifoldFromConstraint(CollisionSolver, Constraint, Dt, BeginPointIndex, Constraint->NumManifoldPoints());

					Constraint->SetSolverBodies(nullptr, nullptr);
				}
			}
		}

	}	// namespace Private
}	// namespace Chaos
