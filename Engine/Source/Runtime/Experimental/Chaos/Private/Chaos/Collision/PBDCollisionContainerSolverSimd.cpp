// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/Collision/PBDCollisionContainerSolverSimd.h"

#include "Chaos/Collision/CollisionApplyType.h"
#include "Chaos/Collision/PBDCollisionConstraint.h"
#include "Chaos/Collision/PBDCollisionSolverUtilities.h"
#include "Chaos/CollisionResolution.h"
#include "Chaos/DebugDrawQueue.h"
#include "Chaos/Evolution/SolverBodyContainer.h"
#include "Chaos/Island/IslandManager.h"
#include "Chaos/Particle/ParticleUtilities.h"
#include "Chaos/PBDCollisionConstraints.h"
#include "Chaos/PBDCollisionConstraintsContact.h"
#include "Chaos/Utilities.h"

// Private includes

#include "ChaosLog.h"

//PRAGMA_DISABLE_OPTIMIZATION

namespace Chaos
{
	namespace CVars
	{
		extern bool bChaos_PBDCollisionSolver_Position_SolveEnabled;
		extern bool bChaos_PBDCollisionSolver_Velocity_SolveEnabled;
	}

	namespace Private
	{

		//////////////////////////////////////////////////////////////////////////////////////////////////
		//////////////////////////////////////////////////////////////////////////////////////////////////
		//////////////////////////////////////////////////////////////////////////////////////////////////
		//////////////////////////////////////////////////////////////////////////////////////////////////

		// Transform and copy a single manifold point for use in the solver
		template<int TNumLanes>
		void UpdateSolverManifoldPointFromConstraint(
			const int32 ManifoldPointIndex,
			TPBDCollisionSolverSimd<4>& Solvers,
			TConstraintPtrSimd<4>& Constraints,
			TSolverBodyPtrPairSimd<4>& Bodies,
			const TArrayView<TPBDCollisionSolverManifoldPointsSimd<TNumLanes>>& ManifoldPointsBuffer,
			const FRealSingle Dt)
		{
			// @todo(chaos): this can be partly simd
			for (int32 LaneIndex = 0; LaneIndex < TNumLanes; ++LaneIndex)
			{
				const FPBDCollisionConstraint* Constraint = Constraints.GetValue(LaneIndex);
				if (Constraint == nullptr)
				{
					continue;
				}

				if (!Constraint->IsManifoldPointActive(ManifoldPointIndex))
				{
					continue;
				}

				const FManifoldPoint& ManifoldPoint = Constraint->GetManifoldPoint(ManifoldPointIndex);
				const FSolverBody& Body0 = *Bodies.Body0.GetValue(LaneIndex);
				const FSolverBody& Body1 = *Bodies.Body1.GetValue(LaneIndex);

				const FRealSingle Restitution = FRealSingle(Constraint->GetRestitution());
				const FRealSingle RestitutionVelocityThreshold = FRealSingle(Constraint->GetRestitutionThreshold()) * Dt;

				// World-space shape transforms. Manifold data is currently relative to these spaces
				const FRigidTransform3& ShapeWorldTransform0 = Constraint->GetShapeWorldTransform0();
				const FRigidTransform3& ShapeWorldTransform1 = Constraint->GetShapeWorldTransform1();

				// World-space contact points on each shape
				const FVec3 WorldContact0 = ShapeWorldTransform0.TransformPositionNoScale(FVec3(ManifoldPoint.ContactPoint.ShapeContactPoints[0]));
				const FVec3 WorldContact1 = ShapeWorldTransform1.TransformPositionNoScale(FVec3(ManifoldPoint.ContactPoint.ShapeContactPoints[1]));
				const FVec3 WorldContact = FReal(0.5) * (WorldContact0 + WorldContact1);
				const FVec3f WorldRelativeContact0 = FVec3f(WorldContact - Body0.P());
				const FVec3f WorldRelativeContact1 = FVec3f(WorldContact - Body1.P());

				// World-space normal
				const FVec3f WorldContactNormal = FVec3f(ShapeWorldTransform1.TransformVectorNoScale(FVec3(ManifoldPoint.ContactPoint.ShapeContactNormal)));

				// World-space tangents
				FVec3f WorldContactTangentU = FVec3f::CrossProduct(FVec3f(0, 1, 0), WorldContactNormal);
				if (!WorldContactTangentU.Normalize(FRealSingle(UE_KINDA_SMALL_NUMBER)))
				{
					WorldContactTangentU = FVec3f::CrossProduct(FVec3f(1, 0, 0), WorldContactNormal);
					WorldContactTangentU = WorldContactTangentU.GetUnsafeNormal();
				}
				const FVec3f WorldContactTangentV = FVec3f::CrossProduct(WorldContactNormal, WorldContactTangentU);

				// Calculate contact velocity if we will need it below (restitution and/or frist-contact friction)
				const bool bNeedContactVelocity = (!ManifoldPoint.Flags.bHasStaticFrictionAnchor) || (Restitution > FRealSingle(0));
				FVec3f ContactVel = FVec3(0);
				if (bNeedContactVelocity)
				{
					const FVec3f ContactVel0 = Body0.V() + FVec3f::CrossProduct(Body0.W(), WorldRelativeContact0);
					const FVec3f ContactVel1 = Body1.V() + FVec3f::CrossProduct(Body1.W(), WorldRelativeContact1);
					ContactVel = ContactVel0 - ContactVel1;
				}

				// If we have contact data from a previous tick, use it to calculate the lateral position delta we need
				// to apply to move the contacts back to their original relative locations (i.e., to enforce static friction).
				// Otherwise, estimate the friction correction from the contact velocity.
				// NOTE: quadratic shapes use the velocity-based path most of the time, unless the relative motion is very small.
				FVec3f WorldFrictionDelta = FVec3f(0);
				if (ManifoldPoint.Flags.bHasStaticFrictionAnchor)
				{
					const FVec3f FrictionDelta0 = ShapeWorldTransform0.TransformPositionNoScale(FVec3(ManifoldPoint.ShapeAnchorPoints[0])) - WorldContact0;
					const FVec3f FrictionDelta1 = ShapeWorldTransform1.TransformPositionNoScale(FVec3(ManifoldPoint.ShapeAnchorPoints[1])) - WorldContact1;
					WorldFrictionDelta = FrictionDelta0 - FrictionDelta1;
				}
				else
				{
					// @todo(chaos): consider adding a multiplier to the initial contact friction
					WorldFrictionDelta = ContactVel * Dt;
				}

				// The contact point error we are trying to correct in this solver
				const FRealSingle TargetPhi = ManifoldPoint.TargetPhi;
				const FVec3f WorldContactDelta = FVec3f(WorldContact0 - WorldContact1);
				const FRealSingle WorldContactDeltaNormal = FVec3f::DotProduct(WorldContactDelta, WorldContactNormal) - TargetPhi;
				const FRealSingle WorldContactDeltaTangentU = FVec3f::DotProduct(WorldContactDelta + WorldFrictionDelta, WorldContactTangentU);
				const FRealSingle WorldContactDeltaTangentV = FVec3f::DotProduct(WorldContactDelta + WorldFrictionDelta, WorldContactTangentV);

				// The target contact velocity, taking restitution into account
				FRealSingle WorldContactTargetVelocityNormal = FRealSingle(0);
				if (Restitution > FRealSingle(0))
				{
					const FRealSingle ContactVelocityNormal = FVec3f::DotProduct(ContactVel, WorldContactNormal);
					if (ContactVelocityNormal < -RestitutionVelocityThreshold)
					{
						WorldContactTargetVelocityNormal = -Restitution * ContactVelocityNormal;
					}
				}

				const FSolverReal InvMScale0 = FSolverReal(Constraint->GetInvMassScale0());
				const FSolverReal InvMScale1 = FSolverReal(Constraint->GetInvMassScale1());
				const FSolverReal InvIScale0 = FSolverReal(Constraint->GetInvInertiaScale0());
				const FSolverReal InvIScale1 = FSolverReal(Constraint->GetInvInertiaScale1());

				Solvers.SetManifoldPoint(
					ManifoldPointIndex,
					LaneIndex,
					ManifoldPointsBuffer,
					WorldRelativeContact0,
					WorldRelativeContact1,
					WorldContactNormal,
					WorldContactTangentU,
					WorldContactTangentV,
					WorldContactDeltaNormal,
					WorldContactDeltaTangentU,
					WorldContactDeltaTangentV,
					WorldContactTargetVelocityNormal,
					Body0,
					Body1,
					InvMScale0,
					InvIScale0,
					InvMScale1,
					InvIScale1);
			}
		}

		// Transform and copy all of a constraint's manifold point data for use by the solver
		template<int TNumLanes>
		void UpdateSolverManifoldPointsFromConstraint(
			TPBDCollisionSolverSimd<4>& Solvers,
			TConstraintPtrSimd<4>& Constraints,
			TSolverBodyPtrPairSimd<4>& Bodies,
			const TArrayView<TPBDCollisionSolverManifoldPointsSimd<TNumLanes>>& ManifoldPointsBuffer,
			const FSolverReal Dt,
			const int32 ManifoldPointBeginIndex, 
			const int32 ManifoldPointEndIndex)
		{
			Solvers.InitManifoldPoints(ManifoldPointsBuffer);

			// Only calculate state for newly added contacts. Normally this is all of them, but maybe not if incremental collision is used by RBAN.
			// Also we only add active points to the solver's manifold points list
			for (int32 ManifoldPointIndex = ManifoldPointBeginIndex; ManifoldPointIndex < ManifoldPointEndIndex; ++ManifoldPointIndex)
			{
				// Transform the constraint contact data into world space for use by the solver
				// We build this data directly into the solver's world-space contact data which looks a bit odd with "Init" called after but there you go
				UpdateSolverManifoldPointFromConstraint(
					ManifoldPointIndex,
					Solvers,
					Constraints,
					Bodies,
					ManifoldPointsBuffer,
					Dt);
			}
		}

		template<int TNumLanes>
		void UpdateSolverFromConstraint(
			TPBDCollisionSolverSimd<4>& Solvers,
			TConstraintPtrSimd<4>& Constraints,
			TSolverBodyPtrPairSimd<4>& Bodies,
			const TArrayView<TPBDCollisionSolverManifoldPointsSimd<TNumLanes>>& ManifoldPointsBuffer,
			const FSolverReal Dt,
			const FPBDCollisionSolverSettings& SolverSettings,
			bool& bOutPerIterationCollision)
		{
			// Friction values. Static and Dynamic friction are applied in the position solve for most shapes.
			// We can also run in a mode without static friction at all. This is faster but stacking is not possible.
			TSimdRealf<TNumLanes> PositionStaticFriction = TSimdRealf<TNumLanes>::Zero();
			TSimdRealf<TNumLanes> PositionDynamicFriction = TSimdRealf<TNumLanes>::Zero();
			TSimdRealf<TNumLanes> VelocityDynamicFriction = TSimdRealf<TNumLanes>::Zero();
			TSimdRealf<TNumLanes> Stiffness = TSimdRealf<TNumLanes>::Zero();
			TSimdInt32<TNumLanes> NumManifoldPoints = TSimdInt32<TNumLanes>::Zero();
			for (int32 LaneIndex = 0; LaneIndex < TNumLanes; ++LaneIndex)
			{
				const FPBDCollisionConstraint* Constraint = Constraints.GetValue(LaneIndex);
				if (Constraint != nullptr)
				{
					const FSolverReal StaticFriction = FSolverReal(Constraint->GetStaticFriction());
					const FSolverReal DynamicFriction = FSolverReal(Constraint->GetDynamicFriction());
						
					PositionStaticFriction.SetValue(LaneIndex, StaticFriction);
					if (!Constraint->HasQuadraticShape())
					{
						PositionDynamicFriction.SetValue(LaneIndex, DynamicFriction);
					}
					else
					{
						// Quadratic shapes don't use PBD dynamic friction - it has issues at slow speeds where the WxR is
						// less than the position tolerance for friction point matching
						// @todo(chaos): fix PBD dynamic friction on quadratic shapes
						VelocityDynamicFriction.SetValue(LaneIndex, DynamicFriction);
					}

					Stiffness.SetValue(LaneIndex, FSolverReal(Constraint->GetStiffness()));

					NumManifoldPoints.SetValue(LaneIndex, Constraint->NumManifoldPoints());
				}
			}

			// NOTE: NumManifoldPoints includes any disabled points so that the point indices match between
			// the solver and the constraint. 
			Solvers.SetNumManifoldPoints(NumManifoldPoints);

			Solvers.SetFriction(PositionStaticFriction, PositionDynamicFriction, VelocityDynamicFriction);
			Solvers.SetStiffness(Stiffness);

			//bOutPerIterationCollision = (!Constraint->GetUseManifold() || Constraint->GetUseIncrementalCollisionDetection());

			UpdateSolverManifoldPointsFromConstraint(
				Solvers,
				Constraints,
				Bodies,
				ManifoldPointsBuffer,
				Dt,
				0,
				Solvers.GetMaxManifoldPoints());
		}

		template<int TNumLanes>
		void UpdateConstraintFromSolver(
			TConstraintPtrSimd<4>& Constraints,
			TPBDCollisionSolverSimd<4>& Solvers,
			TSolverBodyPtrPairSimd<4>& Bodies,
			const TArrayView<TPBDCollisionSolverManifoldPointsSimd<TNumLanes>>& ManifoldPointsBuffer,
			const FSolverReal Dt)
		{
			// @todo(chaos): this can be partly simd
			for (int32 LaneIndex = 0; LaneIndex < TNumLanes; ++LaneIndex)
			{
				FPBDCollisionConstraint* Constraint = Constraints.GetValue(LaneIndex);
				if (Constraint == nullptr)
				{
					continue;
				}

				Constraint->ResetSolverResults();

				for (int32 ManifoldPointIndex = 0; ManifoldPointIndex < Constraint->NumManifoldPoints(); ++ManifoldPointIndex)
				{
					FSolverVec3 NetPushOut = FSolverVec3(0);
					FSolverVec3 NetImpulse = FSolverVec3(0);
					FSolverReal StaticFrictionRatio = FSolverReal(0);

					if (Constraint->IsManifoldPointActive(ManifoldPointIndex))
					{
						NetPushOut = Solvers.GetNetPushOut(ManifoldPointIndex, LaneIndex, ManifoldPointsBuffer);
						NetImpulse = Solvers.GetNetImpulse(ManifoldPointIndex, LaneIndex, ManifoldPointsBuffer);
						StaticFrictionRatio = Solvers.GetStaticFrictionRatio(ManifoldPointIndex, LaneIndex, ManifoldPointsBuffer);
					}

					// NOTE: We call this even for points we did not run the solver for (but with zero results)
					Constraint->SetSolverResults(ManifoldPointIndex,
						NetPushOut,
						NetImpulse,
						StaticFrictionRatio,
						Dt);
				}
			}
		}


		//////////////////////////////////////////////////////////////////////////////////////////////////
		//////////////////////////////////////////////////////////////////////////////////////////////////
		//////////////////////////////////////////////////////////////////////////////////////////////////
		//////////////////////////////////////////////////////////////////////////////////////////////////


		FPBDCollisionContainerSolverSimd::FPBDCollisionContainerSolverSimd(const FPBDCollisionConstraints& InConstraintContainer, const int32 InPriority)
			: FConstraintContainerSolver(InPriority)
			, ConstraintContainer(InConstraintContainer)
			, NumConstraints(0)
			, AppliedShockPropagation()
			, bPerIterationCollisionDetection(false)
		{
	#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST && INTEL_ISPC
			Private::FPBDCollisionSolverHelperSimd::CheckISPC();
	#endif

			for (int32 LaneIndex = 0; LaneIndex < GetNumLanes(); ++LaneIndex)
			{
				SimdData.DummySolverBody0[LaneIndex] = FSolverBody::MakeInitialized();
				SimdData.DummySolverBody1[LaneIndex] = FSolverBody::MakeInitialized();
			}
		}

		FPBDCollisionContainerSolverSimd::~FPBDCollisionContainerSolverSimd()
		{
		}

		void FPBDCollisionContainerSolverSimd::ResetImpl(const int32 MaxCollisions)
		{
			NumConstraints = 0;

			ConstraintSolverIds.Reset(MaxCollisions);

			// @todo(chaos): This is over-allocating by up to a factor of NumLanes!
			SimdData.SimdSolverBodies.Reset(MaxCollisions);
			SimdData.SimdSolvers.Reset(MaxCollisions);
			SimdData.SimdManifoldPoints.Reset(MaxCollisions);
			SimdData.SimdConstraints.Reset(MaxCollisions);
			AppliedShockPropagation.Init(1, MaxCollisions);

			SimdData.SimdNumConstraints.SetValues(0);
		}

		void FPBDCollisionContainerSolverSimd::AddConstraints()
		{
			// Not supported with RBAN (requires islands)
			check(false);
		}

		void FPBDCollisionContainerSolverSimd::AddConstraints(const TArrayView<Private::FPBDIslandConstraint*>& IslandConstraints)
		{
			// Decide what lane this island goes into: Find the lane with the least constraints in it
			int32 IslandLaneIndex = 0;
			int32 IslandLaneNumConstraints = SimdData.SimdNumConstraints.GetValue(0);
			for (int32 LaneIndex = 1; LaneIndex < GetNumLanes(); ++LaneIndex)
			{
				if (SimdData.SimdNumConstraints.GetValue(LaneIndex) < IslandLaneNumConstraints)
				{
					IslandLaneIndex = LaneIndex;
					IslandLaneNumConstraints = SimdData.SimdNumConstraints.GetValue(LaneIndex);
				}
			}

			// Make sure we have enough constraint rows for these constraints (space is pre-allocated)
			// @todo(chaos): If there are many sleeping constraints, we will be overallocating here.
			if (SimdData.SimdConstraints.Num() < IslandLaneNumConstraints + IslandConstraints.Num())
			{
				SimdData.SimdConstraints.SetNumZeroed(IslandLaneNumConstraints + IslandConstraints.Num(), EAllowShrinking::No);
			}

			// Add all the constraints in the island to the selected lane
			int32 NumActiveConstraints = 0;
			for (Private::FPBDIslandConstraint* IslandConstraint : IslandConstraints)
			{
				// Validate the constraint sleep state matches between the FPBDIslandConstraint and FConstraintHandle
				ensureMsgf(IslandConstraint->IsSleeping() == IslandConstraint->GetConstraint()->IsSleeping(),
					TEXT("Mismatch in constraint sleep state. Particle 0: %s, Particle 1: %s."),
					*(IslandConstraint->GetConstraint()->GetConstrainedParticles()[0]->GetDebugName()),
					*(IslandConstraint->GetConstraint()->GetConstrainedParticles()[1]->GetDebugName()));

				// Filter out sleeping constraints in any partially sleeping island
				if (!IslandConstraint->IsSleeping())
				{
					FConstraintHandle* Constraint = IslandConstraint->GetConstraint();

					// NOTE: We will only ever be given constraints from our container (asserts in non-shipping)
					FPBDCollisionConstraint& CollisionConstraint = Constraint->AsUnsafe<FPBDCollisionConstraintHandle>()->GetContact();

					const int32 SolverIndex = SimdData.SimdNumConstraints.GetValue(IslandLaneIndex);
					SimdData.SimdNumConstraints.SetValue(IslandLaneIndex, SolverIndex + 1);

					// Add the constraint to its island's lane
					SimdData.SimdConstraints[SolverIndex].SetValue(IslandLaneIndex, &CollisionConstraint);

					// Store the mapping from constraint to solver/lane
					ConstraintSolverIds.Add({ SolverIndex, IslandLaneIndex });

					++NumActiveConstraints;
				}
			}

			NumConstraints += NumActiveConstraints;
		}

		bool FPBDCollisionContainerSolverSimd::SortConstraintsByPartitionIndex(const TArray<int32>& PartitionIndices, const TArray<int32>& PrefixSum)
		{
			//@todo(Chaos): Implement constraint color sorting for the SIMD solver
			// This requires sorting constraints before the SIMD lanes are allocated 
			// so that only constraints with the same partition index are sharing SIMD data.
			return false;
		}

		void FPBDCollisionContainerSolverSimd::GetAllConstrainedParticles(TArray<const FGeometryParticleHandle*>& ParticleHandles) const
		{
			//@todo(Chaos): Implement constraint color sorting for the SIMD solver
			// This requires sorting constraints before the SIMD lanes are allocated 
			// so that only constraints with the same partition index are sharing SIMD data.
			ParticleHandles.Reset();
		}

		void FPBDCollisionContainerSolverSimd::CreateSolvers()
		{
			const int32 NumSolvers = SimdData.SimdNumConstraints.GetMaxValue();

			// Allocate the solvers (NOTE: unitialized data)
			SimdData.SimdSolvers.SetNum(NumSolvers, EAllowShrinking::No);

			// Determine how many manifold point rows we need and tell each solver where its points are
			int32 NumManifoldPoints = 0;
			for (int32 SolverIndex = 0; SolverIndex < NumSolvers; ++SolverIndex)
			{
				TConstraintPtrSimd<4>& Constraints = SimdData.SimdConstraints[SolverIndex];

				// How may manifold points do we need for this row of constraints?
				int32 MaxSolverManifoldPoints = 0;
				for (int32 LaneIndex = 0; LaneIndex < 4; ++LaneIndex)
				{
					const FPBDCollisionConstraint* Constraint = Constraints.GetValue(LaneIndex);
					if (Constraint != nullptr)
					{
						MaxSolverManifoldPoints = FMath::Max(MaxSolverManifoldPoints, Constraint->NumManifoldPoints());
					}
				}

				// Assign manifold point buffer indices
				SimdData.SimdSolvers[SolverIndex].SetManifoldPointsBuffer(NumManifoldPoints, MaxSolverManifoldPoints);
				NumManifoldPoints += MaxSolverManifoldPoints;
			}

			// Allocate the manifold point solver rows (NOTE: all data cleared to zero)
			// @todo(chaos): we could be more selective about what gets zeroed
			SimdData.SimdManifoldPoints.SetNumZeroed(NumManifoldPoints, EAllowShrinking::No);


#if !CHAOS_SIMDCOLLISIONSOLVER_USEDUMMYSOLVERBODY
			// Allocate the body pointers (NOTE: set to null so that we don't have to visit lanes with no constraint again)
			SimdData.SimdSolverBodies.SetNumZeroed(NumSolvers, EAllowShrinking::No);
#else
			// Allocate the body pointers (NOTE: set to point to a dummy body so that even unused lanes have a valid body)
			SimdData.SimdSolverBodies.SetNumUninitialized(NumSolvers, EAllowShrinking::No);
			for (TSolverBodyPtrPairSimd<4>& BodyPtrPair : SimdData.SimdSolverBodies)
			{
				for (int32 LaneIndex = 0; LaneIndex < GetNumLanes(); ++LaneIndex)
				{
					BodyPtrPair.Body0.SetValue(LaneIndex, &SimdData.DummySolverBody0[LaneIndex]);
					BodyPtrPair.Body1.SetValue(LaneIndex, &SimdData.DummySolverBody1[LaneIndex]);
				}
			}
#endif
		}

		void FPBDCollisionContainerSolverSimd::AddBodies(FSolverBodyContainer& SolverBodyContainer)
		{
			// All constraints and bodies are now known, so we can initialize the array of solvers.
			CreateSolvers();

			for (int32 SolverIndex = 0; SolverIndex < SimdData.SimdSolvers.Num(); ++SolverIndex)
			{
				TConstraintPtrSimd<4>& Constraints = SimdData.SimdConstraints[SolverIndex];
				for (int32 LaneIndex = 0; LaneIndex < 4; ++LaneIndex)
				{
					const FPBDCollisionConstraint* Constraint = Constraints.GetValue(LaneIndex);
					if (Constraint != nullptr)
					{
						FSolverBody* Body0 = SolverBodyContainer.FindOrAdd(Constraint->GetParticle0());
						FSolverBody* Body1 = SolverBodyContainer.FindOrAdd(Constraint->GetParticle1());
						SimdData.SimdSolverBodies[SolverIndex].Body0.SetValue(LaneIndex, Body0);
						SimdData.SimdSolverBodies[SolverIndex].Body1.SetValue(LaneIndex, Body1);
					}
				}
			}
		}

		void FPBDCollisionContainerSolverSimd::GatherInput(const FReal Dt)
		{
			GatherInput(Dt, 0, GetNumConstraints());
		}

		void FPBDCollisionContainerSolverSimd::GatherInput(const FReal InDt, const int32 ConstraintBeginIndex, const int32 ConstraintEndIndex)
		{
			// NOTE: may be called in parallel. Should not change the container or any elements outside of [BeginIndex, EndIndex)

			// Map the constraint range to a row/constraint range
			// @toso(chaos): change the API to ask the container solver for the number of items it needs to gather
			if (ConstraintBeginIndex != 0)
			{
				return;
			}

			const FSolverReal Dt = FSolverReal(InDt);

			TArrayView<TPBDCollisionSolverManifoldPointsSimd<4>> ManifoldPointsBuffer = MakeArrayView(SimdData.SimdManifoldPoints);
			TArrayView<TSolverBodyPtrPairSimd<4>> SolverBodiesBuffer = MakeArrayView(SimdData.SimdSolverBodies);

			bool bAnyPerIterationCollisions = false;
			for (int32 SolverIndex = 0; SolverIndex < SimdData.SimdSolvers.Num(); ++SolverIndex)
			{
				TConstraintPtrSimd<4>& SolverConstraints = SimdData.SimdConstraints[SolverIndex];
				TPBDCollisionSolverSimd<4>& Solvers = SimdData.SimdSolvers[SolverIndex];
				//bool& bPerIterationCollision = bCollisionConstraintPerIterationCollisionDetection[ConstraintIndex];
				bool bPerIterationCollision = false;

				UpdateSolverFromConstraint(
					SimdData.SimdSolvers[SolverIndex],
					SimdData.SimdConstraints[SolverIndex],
					SimdData.SimdSolverBodies[SolverIndex],
					ManifoldPointsBuffer,
					Dt,
					ConstraintContainer.GetSolverSettings(),
					bPerIterationCollision);

				bAnyPerIterationCollisions = bAnyPerIterationCollisions || bPerIterationCollision;
			}

			if (bAnyPerIterationCollisions)
			{
				// We should lock here? We only ever set to true or do nothing so I think it doesn't matter if this happens on multiple threads...
				bPerIterationCollisionDetection = true;
			}
		}

		void FPBDCollisionContainerSolverSimd::ScatterOutput(const FReal Dt)
		{
			ScatterOutput(Dt, 0, GetNumConstraints());
		}

		void FPBDCollisionContainerSolverSimd::ScatterOutput(const FReal InDt, const int32 ConstraintBeginIndex, const int32 ConstraintEndIndex)
		{
			// NOTE: may be called in parallel. Should not change the container or any elements outside of [BeginIndex, EndIndex)

			// Map the constraint range to a row/constraint range
			// @toso(chaos): change the API to ask the container solver for the number of items it needs to gather
			if (ConstraintBeginIndex != 0)
			{
				return;
			}

			const FSolverReal Dt = FSolverReal(InDt);

			TArrayView<TPBDCollisionSolverManifoldPointsSimd<4>> ManifoldPointsBuffer = MakeArrayView(SimdData.SimdManifoldPoints);

			// @todo(chaos): would it be better to iterate over manifold point rows?
			for (int32 SolverIndex = 0; SolverIndex < SimdData.SimdSolvers.Num(); ++SolverIndex)
			{
				TConstraintPtrSimd<4>& SolverConstraints = SimdData.SimdConstraints[SolverIndex];
				TPBDCollisionSolverSimd<4>& Solvers = SimdData.SimdSolvers[SolverIndex];

				UpdateConstraintFromSolver(
					SimdData.SimdConstraints[SolverIndex],
					SimdData.SimdSolvers[SolverIndex],
					SimdData.SimdSolverBodies[SolverIndex],
					ManifoldPointsBuffer,
					Dt);
			}
		}

		void FPBDCollisionContainerSolverSimd::ApplyPositionConstraints(const FReal InDt, const int32 It, const int32 NumIts)
		{
			ApplyPositionConstraints(InDt, It, NumIts, 0, SimdData.SimdSolvers.Num());
		}

		void FPBDCollisionContainerSolverSimd::ApplyPositionConstraints(const FReal InDt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex)
		{
			SCOPE_CYCLE_COUNTER(STAT_Collisions_Apply);
			if (!CVars::bChaos_PBDCollisionSolver_Position_SolveEnabled)
			{
				return;
			}

			const FPBDCollisionSolverSettings& SolverSettings = ConstraintContainer.GetSolverSettings();
			const FSolverReal Dt = FSolverReal(InDt);

			UpdatePositionShockPropagation(It, NumIts, BeginIndex, EndIndex, SolverSettings);

			// We run collision detection here under two conditions (normally it is run after Integration and before the constraint solver phase):
			// 1) When deferring collision detection until the solver phase for better joint-collision behaviour (RBAN). In this case, we only do this on the first iteration.
			// 2) When using no manifolds or incremental manifolds, where we may add/replace manifold points every iteration.
			const bool bRunDeferredCollisionDetection = (It == 0) && ConstraintContainer.GetDetectorSettings().bDeferNarrowPhase;
			if (bRunDeferredCollisionDetection || bPerIterationCollisionDetection)
			{
				UpdateCollisions(Dt, BeginIndex, EndIndex);
			}

			// Only apply friction for the last few (tunable) iterations
			// Adjust max pushout to attempt to make it iteration count independent
			const bool bApplyStaticFriction = (It >= (NumIts - SolverSettings.NumPositionFrictionIterations));
			const FSolverReal MaxPushOut = (SolverSettings.MaxPushOutVelocity > 0) ? (FSolverReal(SolverSettings.MaxPushOutVelocity) * Dt) / FSolverReal(NumIts) : FSolverReal(0);

			// Apply the position correction
			if (bApplyStaticFriction)
			{
				Private::FPBDCollisionSolverHelperSimd::SolvePositionWithFriction(
					MakeArrayView(&SimdData.SimdSolvers[BeginIndex], EndIndex - BeginIndex),
					MakeArrayView(&SimdData.SimdManifoldPoints[BeginIndex], EndIndex - BeginIndex),
					MakeArrayView(&SimdData.SimdSolverBodies[BeginIndex], EndIndex - BeginIndex),
					Dt,
					MaxPushOut);
			}
			else
			{
				Private::FPBDCollisionSolverHelperSimd::SolvePositionNoFriction(
					MakeArrayView(&SimdData.SimdSolvers[BeginIndex], EndIndex - BeginIndex),
					MakeArrayView(&SimdData.SimdManifoldPoints[BeginIndex], EndIndex - BeginIndex),
					MakeArrayView(&SimdData.SimdSolverBodies[BeginIndex], EndIndex - BeginIndex),
					Dt,
					MaxPushOut);
			}
		}

		void FPBDCollisionContainerSolverSimd::ApplyVelocityConstraints(const FReal InDt, const int32 It, const int32 NumIts)
		{
			ApplyVelocityConstraints(InDt, It, NumIts, 0, SimdData.SimdSolvers.Num());
		}

		void FPBDCollisionContainerSolverSimd::ApplyVelocityConstraints(const FReal InDt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex)
		{
			SCOPE_CYCLE_COUNTER(STAT_Collisions_ApplyPushOut);
			if (!CVars::bChaos_PBDCollisionSolver_Velocity_SolveEnabled)
			{
				return;
			}

			const FPBDCollisionSolverSettings& SolverSettings = ConstraintContainer.GetSolverSettings();

			UpdateVelocityShockPropagation(It, NumIts, BeginIndex, EndIndex, SolverSettings);

			const FSolverReal Dt = FSolverReal(InDt);
			const bool bApplyDynamicFriction = (It >= NumIts - SolverSettings.NumVelocityFrictionIterations);

			if (bApplyDynamicFriction)
			{
				Private::FPBDCollisionSolverHelperSimd::SolveVelocityWithFriction(
					MakeArrayView(&SimdData.SimdSolvers[BeginIndex], EndIndex - BeginIndex),
					MakeArrayView(&SimdData.SimdManifoldPoints[BeginIndex], EndIndex - BeginIndex),
					MakeArrayView(&SimdData.SimdSolverBodies[BeginIndex], EndIndex - BeginIndex),
					Dt);
			}
			else
			{
				Private::FPBDCollisionSolverHelperSimd::SolveVelocityNoFriction(
					MakeArrayView(&SimdData.SimdSolvers[BeginIndex], EndIndex - BeginIndex),
					MakeArrayView(&SimdData.SimdManifoldPoints[BeginIndex], EndIndex - BeginIndex),
					MakeArrayView(&SimdData.SimdSolverBodies[BeginIndex], EndIndex - BeginIndex),
					Dt);
			}
		}

		void FPBDCollisionContainerSolverSimd::ApplyProjectionConstraints(const FReal Dt, const int32 It, const int32 NumIts)
		{
			ApplyProjectionConstraints(Dt, It, NumIts, 0, SimdData.SimdSolvers.Num());
		}

		void FPBDCollisionContainerSolverSimd::ApplyProjectionConstraints(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex)
		{
			// Not supported for collisions
		}

		void FPBDCollisionContainerSolverSimd::ApplyShockPropagation(const FSolverReal ShockPropagation, const int32 BeginIndex, const int32 EndIndex)
		{
			// @todo(chaos): cache the mass scales so we don't have to look in the constraint again
			for (int32 SolverIndex = BeginIndex; SolverIndex < EndIndex; ++SolverIndex)
			{
				if (ShockPropagation != AppliedShockPropagation[SolverIndex])
				{
					for (int32 LaneIndex = 0; LaneIndex < 4; ++LaneIndex)
					{
						const FPBDCollisionConstraint* Constraint = SimdData.SimdConstraints[SolverIndex].GetValue(LaneIndex);
						if (Constraint != nullptr)
						{
							const FSolverBody* Body0 = SimdData.SimdSolverBodies[SolverIndex].Body0.GetValue(LaneIndex);
							const FSolverBody* Body1 = SimdData.SimdSolverBodies[SolverIndex].Body1.GetValue(LaneIndex);

							FSolverReal ShockPropagation0, ShockPropagation1;
							if (CalculateBodyShockPropagation(*Body0, *Body1, ShockPropagation, ShockPropagation0, ShockPropagation1))
							{
								SimdData.SimdSolvers[SolverIndex].UpdateMassNormal(
									LaneIndex,
									MakeArrayView(SimdData.SimdManifoldPoints),
									*Body0,
									*Body1,
									ShockPropagation0 * FSolverReal(Constraint->GetInvMassScale0()),
									ShockPropagation0 * FSolverReal(Constraint->GetInvInertiaScale0()),
									ShockPropagation1 * FSolverReal(Constraint->GetInvMassScale1()),
									ShockPropagation1 * FSolverReal(Constraint->GetInvInertiaScale1()));
							}
						}
					}
					AppliedShockPropagation[SolverIndex] = ShockPropagation;
				}
			}
		}

		void FPBDCollisionContainerSolverSimd::UpdatePositionShockPropagation(const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex, const FPBDCollisionSolverSettings& SolverSettings)
		{
			const bool bEnableShockPropagation = (It >= NumIts - SolverSettings.NumPositionShockPropagationIterations);
			const FSolverReal ShockPropagation = (bEnableShockPropagation) ? CVars::Chaos_PBDCollisionSolver_Position_MinInvMassScale : FSolverReal(1);
			ApplyShockPropagation(ShockPropagation, BeginIndex, EndIndex);
		}

		void FPBDCollisionContainerSolverSimd::UpdateVelocityShockPropagation(const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex, const FPBDCollisionSolverSettings& SolverSettings)
		{
			const bool bEnableShockPropagation = (It >= NumIts - SolverSettings.NumVelocityShockPropagationIterations);
			const FSolverReal ShockPropagation = (bEnableShockPropagation) ? CVars::Chaos_PBDCollisionSolver_Velocity_MinInvMassScale : FSolverReal(1);
			ApplyShockPropagation(ShockPropagation, BeginIndex, EndIndex);
		}

		void FPBDCollisionContainerSolverSimd::UpdateCollisions(const FSolverReal InDt, const int32 BeginIndex, const int32 EndIndex)
		{
			//const FSolverReal Dt = FSolverReal(InDt);
			//const bool bDeferredCollisionDetection = ConstraintContainer.GetDetectorSettings().bDeferNarrowPhase;

			//bool bNeedsAnotherIteration = false;
			//for (int32 SolverIndex = BeginIndex; SolverIndex < EndIndex; ++SolverIndex)
			//{
			//	if (bDeferredCollisionDetection || bCollisionConstraintPerIterationCollisionDetection[SolverIndex])
			//	{
			//		Private::FPBDCollisionSolverSimd& CollisionSolver = Solvers[SolverIndex];
			//		FPBDCollisionConstraint* Constraint = Constraints[SolverIndex];

			//		// Run collision detection at the current transforms including any correction from previous iterations
			//		const FSolverBody& Body0 = CollisionSolver.SolverBody0().SolverBody();
			//		const FSolverBody& Body1 = CollisionSolver.SolverBody1().SolverBody();
			//		const FRigidTransform3 CorrectedActorWorldTransform0 = FRigidTransform3(Body0.CorrectedActorP(), Body0.CorrectedActorQ());
			//		const FRigidTransform3 CorrectedActorWorldTransform1 = FRigidTransform3(Body1.CorrectedActorP(), Body1.CorrectedActorQ());
			//		const FRigidTransform3 CorrectedShapeWorldTransform0 = Constraint->GetShapeRelativeTransform0() * CorrectedActorWorldTransform0;
			//		const FRigidTransform3 CorrectedShapeWorldTransform1 = Constraint->GetShapeRelativeTransform1() * CorrectedActorWorldTransform1;

			//		// @todo(chaos): this is ugly - pass these to the required functions instead and remove from the constraint class
			//		// This is now only needed for LevelSet collision (see UpdateLevelsetLevelsetConstraint)
			//		Constraint->SetSolverBodies(&Body0, &Body1);

			//		// Reset the manifold if we are not using manifolds (we just use the first manifold point)
			//		if (!Constraint->GetUseManifold())
			//		{
			//			Constraint->ResetActiveManifoldContacts();
			//			CollisionSolver.ResetManifold();
			//		}

			//		// We need to know how many points were added to the manifold
			//		const int32 BeginPointIndex = Constraint->NumManifoldPoints();

			//		// NOTE: We deliberately have not updated the ShapwWorldTranforms on the constraint. If we did that, we would calculate 
			//		// errors incorrectly in UpdateManifoldPoints, because the solver assumes nothing has been moved as we iterate (we accumulate 
			//		// corrections that will be applied later.)
			//		Constraint->ResetPhi(Constraint->GetCullDistance());
			//		Collisions::UpdateConstraint(*Constraint, CorrectedShapeWorldTransform0, CorrectedShapeWorldTransform1, Dt);

			//		// Update the manifold based on the new or updated contacts
			//		UpdateSolverManifoldFromConstraint(
			//			MakeArrayView(SimdData.SimdManifoldPoints),
			//			MakeArrayView(SimdData.SimdSolverBodies),
			//			CollisionSolver, 
			//			Constraint, 
			//			Dt, 
			//			BeginPointIndex, 
			//			Constraint->NumManifoldPoints());

			//		Constraint->SetSolverBodies(nullptr, nullptr);
			//	}
			//}
		}

	}	// namespace Private
}	// namespace Chaos
