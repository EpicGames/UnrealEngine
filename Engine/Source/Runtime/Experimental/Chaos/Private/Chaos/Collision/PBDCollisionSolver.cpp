// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/Collision/PBDCollisionSolver.h"

#include "Chaos/Collision/CollisionApplyType.h"
#include "Chaos/Collision/PBDCollisionConstraint.h"
#include "Chaos/CollisionResolution.h"
#include "Chaos/CollisionResolutionTypes.h"
#include "Chaos/CollisionResolutionUtil.h"
#include "Chaos/Evolution/SolverBodyContainer.h"
#include "Chaos/Particle/ParticleUtilities.h"
#include "Chaos/PBDCollisionConstraints.h"
#include "Chaos/PBDCollisionConstraintsContact.h"
#include "Chaos/Utilities.h"

#if INTEL_ISPC
#include "PBDCollisionSolver.ispc.generated.h"
#endif

//PRAGMA_DISABLE_OPTIMIZATION

// TEMP: to be removed
DECLARE_CYCLE_STAT(TEXT("SolvePositionNoFriction"), STAT_SolvePositionNoFriction, STATGROUP_ChaosConstraintSolver);

namespace Chaos
{
	namespace CVars
	{
		//
		// Position Solver Settings
		//

		bool bChaos_PBDCollisionSolver_Position_SolveEnabled = true;
		float Chaos_PBDCollisionSolver_Position_MinInvMassScale = 0.77f;
		float Chaos_PBDCollisionSolver_Position_StaticFrictionStiffness = 0.5f;

		FAutoConsoleVariableRef CVarChaos_PBDCollisionSolver_Position_SolveEnabled(TEXT("p.Chaos.PBDCollisionSolver.Position.SolveEnabled"), bChaos_PBDCollisionSolver_Position_SolveEnabled, TEXT(""));
		FAutoConsoleVariableRef CVarChaos_PBDCollisionSolver_Position_MinInvMassScale(TEXT("p.Chaos.PBDCollisionSolver.Position.MinInvMassScale"), Chaos_PBDCollisionSolver_Position_MinInvMassScale, TEXT(""));
		FAutoConsoleVariableRef CVarChaos_PBDCollisionSolver_Position_StaticFrictionStiffness(TEXT("p.Chaos.PBDCollisionSolver.Position.StaticFriction.Stiffness"), Chaos_PBDCollisionSolver_Position_StaticFrictionStiffness, TEXT(""));

		//
		// Velocity Solver Settings
		//

		bool bChaos_PBDCollisionSolver_Velocity_SolveEnabled = true;
		// If Chaos_PBDCollisionSolver_Velocity_MinInvMassScale is the same as Chaos_PBDCollisionSolver_Position_MinInvMassScale and all velocity iterations have shockpropagation, we avoid recalculating constraint-space mass
		float Chaos_PBDCollisionSolver_Velocity_MinInvMassScale = Chaos_PBDCollisionSolver_Position_MinInvMassScale;
		bool bChaos_PBDCollisionSolver_Velocity_FrictionEnabled = true;
		float Chaos_PBDCollisionSolver_Velocity_StaticFrictionStiffness = 1.0f;
		bool bChaos_PBDCollisionSolver_Velocity_AveragePointEnabled = false;

		FAutoConsoleVariableRef CVarChaos_PBDCollisionSolver_Velocity_SolveEnabled(TEXT("p.Chaos.PBDCollisionSolver.Velocity.SolveEnabled"), bChaos_PBDCollisionSolver_Velocity_SolveEnabled, TEXT(""));
		FAutoConsoleVariableRef CVarChaos_PBDCollisionSolver_Velocity_MinInvMassScale(TEXT("p.Chaos.PBDCollisionSolver.Velocity.MinInvMassScale"), Chaos_PBDCollisionSolver_Velocity_MinInvMassScale, TEXT(""));
		FAutoConsoleVariableRef CVarChaos_PBDCollisionSolver_Velocity_FrictionEnabled(TEXT("p.Chaos.PBDCollisionSolver.Velocity.FrictionEnabled"), bChaos_PBDCollisionSolver_Velocity_FrictionEnabled, TEXT(""));
		FAutoConsoleVariableRef CVarChaos_PBDCollisionSolver_Velocity_StaticFrictionStiffness(TEXT("p.Chaos.PBDCollisionSolver.Velocity.StaticFriction.Stiffness"), Chaos_PBDCollisionSolver_Velocity_StaticFrictionStiffness, TEXT(""));
		FAutoConsoleVariableRef CVarChaos_PBDCollisionSolver_Velocity_AveragePointEnabled(TEXT("p.Chaos.PBDCollisionSolver.Velocity.AveragePointEnabled"), bChaos_PBDCollisionSolver_Velocity_AveragePointEnabled, TEXT(""));

		bool bChaos_PBDCollisionSolver_Use1DFriction = false;
		FAutoConsoleVariableRef CVarChaos_PBDCollisionSolver_CVarUse1DFriction(TEXT("p.Chaos.Solver.Use1DFriction"), bChaos_PBDCollisionSolver_Use1DFriction, TEXT("Use 1 dimensional friction along the velocity axis."));
	}
	using namespace CVars;

	namespace Private
	{
		void FPBDCollisionSolver::SolveVelocityAverage(const FSolverReal Dt)
		{
			// @todo(chaos): A better solution would be to add an extra manifold point into array and use it for the position solver as well.
			FSolverReal InvM0, InvM1;
			FSolverMatrix33 InvI0, InvI1;
			GetDynamicMassProperties(InvM0, InvI0, InvM1, InvI1);

			// Generate a new contact point at the average of all the active contacts
			int32 NumActiveManifoldPoints = 0;
			FSolverVec3 RelativeContactPosition0 = FSolverVec3(0);
			FSolverVec3 RelativeContactPosition1 = FSolverVec3(0);
			FSolverVec3 WorldContactNormal = FSolverVec3(0);
			FSolverReal NetPushOutNormal = FSolverReal(0);
			FSolverReal WorldContactVelocityTargetNormal = FSolverReal(0);
			for (int32 PointIndex = 0; PointIndex < NumManifoldPoints(); ++PointIndex)
			{
				FPBDCollisionSolverManifoldPoint& SolverManifoldPoint = State.ManifoldPoints[PointIndex];
				if (ShouldSolveVelocity(SolverManifoldPoint))
				{
					RelativeContactPosition0 += SolverManifoldPoint.RelativeContactPoints[0];
					RelativeContactPosition1 += SolverManifoldPoint.RelativeContactPoints[1];
					WorldContactVelocityTargetNormal += SolverManifoldPoint.ContactTargetVelocityNormal;
					WorldContactNormal = SolverManifoldPoint.ContactNormal;	// Take last value - should all be similar
					NetPushOutNormal += SolverManifoldPoint.NetPushOutNormal;
					++NumActiveManifoldPoints;
				}
			}

			// Solve for velocity at the new conatct point
			// We only do this if we have multiple active contacts
			// NOTE: this average contact isn't really correct, especially when all contacts do not equally
			// contribute to the pushout (which is normal even for a simple box on a plane). E.g., the WorldContactVelocityTargetNormal
			// and NetPushOutNormal will not be right. The goal here though is just to get close to the correct result in fewer iterations
			// than we would have if we just solved the corners of the box in sequence.
			if (NumActiveManifoldPoints > 1)
			{
				const FSolverReal InvCount = FSolverReal(1) / FSolverReal(NumActiveManifoldPoints);

				FPBDCollisionSolverManifoldPoint AverageManifoldPoint;
				AverageManifoldPoint.RelativeContactPoints[0] = RelativeContactPosition0 * InvCount;
				AverageManifoldPoint.RelativeContactPoints[1] = RelativeContactPosition1 * InvCount;
				AverageManifoldPoint.ContactNormal = WorldContactNormal;
				AverageManifoldPoint.ContactTargetVelocityNormal = WorldContactVelocityTargetNormal * InvCount;
			
				// Total pushout (not average) which is correct if the average point is also the centroid but otherwise probably an overestimate.
				// This is used to limit the possibly-attractive impulse that corrects implicit velocity errors from the PBD solve, but it
				// which might cause stickiness if overestimated. Although it should be recitifed when we solve the normal contact point velocities
				// so I don't think it matters...we could do better though if it is a problem
				AverageManifoldPoint.NetPushOutNormal = NetPushOutNormal;

				const FSolverVec3 R0xN = FSolverVec3::CrossProduct(RelativeContactPosition0, WorldContactNormal);
				AverageManifoldPoint.ContactRxNormal0 = R0xN;

				const FSolverVec3 R1xN = FSolverVec3::CrossProduct(RelativeContactPosition1, WorldContactNormal);
				AverageManifoldPoint.ContactRxNormal1 = R1xN;

				// Calculate the contact mass (and derived properties) for this point
				FSolverVec3 WorldContactNormalAngular0 = FSolverVec3(0);
				FSolverVec3 WorldContactNormalAngular1 = FSolverVec3(0);
				FSolverReal ContactMassInvNormal = FSolverReal(0);
				if (IsDynamic(0))
				{
					WorldContactNormalAngular0 = InvI0 * R0xN;
					ContactMassInvNormal += FSolverVec3::DotProduct(R0xN, WorldContactNormalAngular0) + InvM0;
				}
				if (IsDynamic(1))
				{
					WorldContactNormalAngular1 = InvI1 * R1xN;
					ContactMassInvNormal += FSolverVec3::DotProduct(R1xN, WorldContactNormalAngular1) + InvM1;
				}
				AverageManifoldPoint.ContactMassNormal = (ContactMassInvNormal > FSolverReal(UE_SMALL_NUMBER)) ? FSolverReal(1) / ContactMassInvNormal : FSolverReal(0);
				AverageManifoldPoint.ContactNormalAngular0 = WorldContactNormalAngular0;
				AverageManifoldPoint.ContactNormalAngular1 = WorldContactNormalAngular1;

				// @todo(chaos): we don't use these - maybe do the calculation without an actual manifold point object...
				AverageManifoldPoint.ContactTangentU = FSolverVec3(0);
				AverageManifoldPoint.ContactTangentV = FSolverVec3(0);
				AverageManifoldPoint.ContactDeltaNormal = FSolverReal(0);
				AverageManifoldPoint.ContactDeltaTangentU = FSolverReal(0);
				AverageManifoldPoint.ContactDeltaTangentV = FSolverReal(0);
				AverageManifoldPoint.ContactRxTangentU0 = FVec3(0);
				AverageManifoldPoint.ContactRxTangentV0 = FVec3(0);
				AverageManifoldPoint.ContactRxTangentU1 = FVec3(0);
				AverageManifoldPoint.ContactRxTangentV1 = FVec3(0);
				AverageManifoldPoint.ContactTangentUAngular0 = FSolverVec3(0);
				AverageManifoldPoint.ContactTangentVAngular0 = FSolverVec3(0);
				AverageManifoldPoint.ContactTangentUAngular1 = FSolverVec3(0);
				AverageManifoldPoint.ContactTangentVAngular1 = FSolverVec3(0);
				AverageManifoldPoint.ContactMassTangentU = FSolverReal(0);
				AverageManifoldPoint.ContactMassTangentV = FSolverReal(0);
				AverageManifoldPoint.NetPushOutTangentU = FSolverReal(0);
				AverageManifoldPoint.NetPushOutTangentV = FSolverReal(0);
				AverageManifoldPoint.NetImpulseNormal = FSolverReal(0);
				AverageManifoldPoint.NetImpulseTangentU = FSolverReal(0);
				AverageManifoldPoint.NetImpulseTangentV = FSolverReal(0);
				AverageManifoldPoint.StaticFrictionRatio = FSolverReal(0);

				FSolverReal ContactVelocityDeltaNormal;
				CalculateContactVelocityErrorNormal(AverageManifoldPoint, ContactVelocityDeltaNormal);

				const FSolverReal MinImpulseNormal = FMath::Min(FSolverReal(0), -AverageManifoldPoint.NetPushOutNormal / Dt);

				ApplyVelocityCorrectionNormal(
					State.Stiffness,
					ContactVelocityDeltaNormal,
					MinImpulseNormal,
					AverageManifoldPoint);

				// Now distribute the net impulse among the active points so we don't over-correct pushout from initial overlaps
				for (int32 PointIndex = 0; PointIndex < NumManifoldPoints(); ++PointIndex)
				{
					FPBDCollisionSolverManifoldPoint& SolverManifoldPoint = State.ManifoldPoints[PointIndex];
					if (ShouldSolveVelocity(SolverManifoldPoint))
					{
						SolverManifoldPoint.NetImpulseNormal += AverageManifoldPoint.NetImpulseNormal * InvCount;
					}
				}
			}
		}

	}	// namespace Private
}	// namespace Chaos
