// Copyright Epic Games, Inc. All Rights Reserved.

#include "Math/Vector.isph"
#include "Math/Quat.isph"
#include "Math/Transform.isph"
#include "Chaos/Matrix33.isph"

//
//
// STRUCTURES
//
//

export struct FSolverBody
{
	// Net position delta applied by all constraints (constantly changing as we iterate over constraints)
	FVector3f DP;
	float DPPad;

	// Net rotation delta applied by all constraints (constantly changing as we iterate over constraints)
	FVector3f DQ;
	float DQPad;

	// World-space center of mass velocity
	FVector3f V;
	float VPad;

	// World-space center of mass angular velocity
	FVector3f W;
	float WPad;

	// World-space inverse inertia
	FMatrix33f InvI;

	// Actor-space center of mass rotation
	FVector4d RoM;

	// World-space rotation of mass at start of sub step
	FVector4d R;

	// Predicted world-space center of mass rotation (post-integration, pre-constraint-solve)
	FVector4d Q;

	// Actor-space center of mass location
	FVector3d CoM;

	// World-space center of mass state at start of sub step
	FVector3d X;

	// Predicted world-space center of mass position (post-integration, pre-constraint-solve)
	FVector3d P;


	// Local-space inverse inertia (diagonal, so only 3 elements)
	FVector3f InvILocal;

	// Inverse mass
	float InvM;

	// Net position correction delta applied by all constraints (constantly changing as we iterate over constraints)
	// Will translate the body without introducing linear velocity
	FVector3f CP;

	// Net rotation correction delta applied by all constraints (constantly changing as we iterate over constraints)
	// Will rotate the body without introducing angular velocity
	FVector3f CQ;

	// Distance to a kinematic body (through the contact graph). Used by collision shock propagation
	int32 Level;

	float AlignmentPadding[2];
};

export struct FWorldContactPoint
{
	// World-space contact point relative to each particle's center of mass
	FVector3f RelativeContactPoints[2];

	// World-space contact normal and tangents
	FVector3f ContactNormal;
	FVector3f ContactTangentU;
	FVector3f ContactTangentV;

	// Errors to correct along each of the contact axes
	float ContactDeltaNormal;
	float ContactDeltaTangentU;
	float ContactDeltaTangentV;

	// Target velocity along the normal direction
	float ContactTargetVelocityNormal;
};

export struct FPBDCollisionSolverManifoldPoint
{
	// World-space contact data
	FWorldContactPoint WorldContact;

	// I^-1.(R x A) for each body where A is each axis (Normal, TangentU, TangentV)
	FVector3f WorldContactNormalAngular0;
	FVector3f WorldContactTangentUAngular0;
	FVector3f WorldContactTangentVAngular0;
	FVector3f WorldContactNormalAngular1;
	FVector3f WorldContactTangentUAngular1;
	FVector3f WorldContactTangentVAngular1;

	// Contact mass (for non-friction)
	float ContactMassNormal;
	float ContactMassTangentU;
	float ContactMassTangentV;

	// Solver outputs
	float NetPushOutNormal;
	float NetPushOutTangentU;
	float NetPushOutTangentV;
	float NetImpulseNormal;
	float NetImpulseTangentU;
	float NetImpulseTangentV;

	// A measure of how much we exceeded the static friction threshold.
	// Equal to (NormalPushOut / TangentialPushOut) before clamping to the friction cone.
	// Used to move the static friction anchors to the edge of the cone in Scatter.
	float StaticFrictionRatio;
};

export struct FPBDCollisionSolver
{
	// Static Friction in the position-solve phase
	float StaticFriction;

	// Dynamic Friction in the position-solve phase
	float DynamicFriction;

	// Dynamic Friction in the velocity-solve phase
	float VelocityFriction;

	// Solver stiffness (scales all pushout and impulses)
	float Stiffness;

	// Bodies and contacts
	FSolverBody* SolverBodies[2];
	float InvMs[2];
	FPBDCollisionSolverManifoldPoint* ManifoldPoints;
	int32 NumManifoldPoints;
	int32 MaxManifoldPoints;
};

export void SolvePositionNoFriction()
{
}