// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/Collision/PBDCollisionSolverJacobi.h"

#include "Chaos/Collision/CollisionApplyType.h"
#include "Chaos/Collision/PBDCollisionConstraint.h"
#include "Chaos/CollisionResolution.h"
#include "Chaos/CollisionResolutionTypes.h"
#include "Chaos/CollisionResolutionUtil.h"
#include "Chaos/Evolution/SolverBodyContainer.h"
#include "Chaos/Particle/ParticleUtilities.h"
#include "Chaos/PBDCollisionConstraints.h"
#include "Chaos/PBDCollisionConstraintsContact.h"
#include "Chaos/Utilities.h"

//PRAGMA_DISABLE_OPTIMIZATION

namespace Chaos
{
	namespace CVars
	{
		extern float Chaos_PBDCollisionSolver_Position_MinInvMassScale;
		extern float Chaos_PBDCollisionSolver_Velocity_MinInvMassScale;
	}

	namespace Private
	{
		//////////////////////////////////////////////////////////////////////////////////////////////////
		//////////////////////////////////////////////////////////////////////////////////////////////////
		//////////////////////////////////////////////////////////////////////////////////////////////////
		//////////////////////////////////////////////////////////////////////////////////////////////////


		void FPBDCollisionSolverJacobi::EnablePositionShockPropagation()
		{
			SetShockPropagationInvMassScale(CVars::Chaos_PBDCollisionSolver_Position_MinInvMassScale);
		}

		void FPBDCollisionSolverJacobi::EnableVelocityShockPropagation()
		{
			SetShockPropagationInvMassScale(CVars::Chaos_PBDCollisionSolver_Velocity_MinInvMassScale);
		}

		void FPBDCollisionSolverJacobi::DisableShockPropagation()
		{
			SetShockPropagationInvMassScale(FReal(1));
		}

		void FPBDCollisionSolverJacobi::SetShockPropagationInvMassScale(const FSolverReal InvMassScale)
		{
			FConstraintSolverBody& Body0 = SolverBody0();
			FConstraintSolverBody& Body1 = SolverBody1();

			// Shock propagation decreases the inverse mass of bodies that are lower in the pile
			// of objects. This significantly improves stability of heaps and stacks. Height in the pile is indictaed by the "level". 
			// No need to set an inverse mass scale if the other body is kinematic (with inv mass of 0).
			// Bodies at the same level do not take part in shock propagation.
			if (Body0.IsDynamic() && Body1.IsDynamic() && (Body0.Level() != Body1.Level()))
			{
				// Set the inv mass scale of the "lower" body to make it heavier
				bool bInvMassUpdated = false;
				if (Body0.Level() < Body1.Level())
				{
					if (Body0.ShockPropagationScale() != InvMassScale)
					{
						Body0.SetShockPropagationScale(InvMassScale);
						bInvMassUpdated = true;
					}
				}
				else
				{
					if (Body1.ShockPropagationScale() != InvMassScale)
					{
						Body1.SetShockPropagationScale(InvMassScale);
						bInvMassUpdated = true;
					}
				}

				// If the masses changed, we need to rebuild the contact mass for each manifold point
				if (bInvMassUpdated)
				{
					for (int32 PointIndex = 0; PointIndex < NumManifoldPoints(); ++PointIndex)
					{
						State.ManifoldPoints.UpdateMassNormal(PointIndex, Body0, Body1);
					}
				}
			}
		}

		//////////////////////////////////////////////////////////////////////////////////////////////////
		//////////////////////////////////////////////////////////////////////////////////////////////////
		//////////////////////////////////////////////////////////////////////////////////////////////////
		//////////////////////////////////////////////////////////////////////////////////////////////////

		void FPBDCollisionSolverJacobiHelper::SolvePositionNoFriction(const TArrayView<FPBDCollisionSolverJacobi>& CollisionSolvers, const FSolverReal Dt, const FSolverReal MaxPushOut)
		{
			//SCOPE_CYCLE_COUNTER(STAT_SolvePositionNoFriction);

			for (FPBDCollisionSolverJacobi& CollisionSolver : CollisionSolvers)
			{
				CollisionSolver.SolvePositionNoFriction(Dt, MaxPushOut);
			}
		}

		void FPBDCollisionSolverJacobiHelper::SolvePositionWithFriction(const TArrayView<FPBDCollisionSolverJacobi>& CollisionSolvers, const FSolverReal Dt, const FSolverReal MaxPushOut)
		{
			for (FPBDCollisionSolverJacobi& CollisionSolver : CollisionSolvers)
			{
				CollisionSolver.SolvePositionWithFriction(Dt, MaxPushOut);
			}
		}

		void FPBDCollisionSolverJacobiHelper::SolveVelocity(const TArrayView<FPBDCollisionSolverJacobi>& CollisionSolvers, const FSolverReal Dt, const bool bApplyDynamicFriction)
		{
			for (FPBDCollisionSolverJacobi& CollisionSolver : CollisionSolvers)
			{
				CollisionSolver.SolveVelocity(Dt, bApplyDynamicFriction);
			}
		}

		void FPBDCollisionSolverJacobiHelper::CheckISPC()
		{
#if INTEL_ISPC
#endif
		}

	}	// namespace Private
}	// namespace Chaos