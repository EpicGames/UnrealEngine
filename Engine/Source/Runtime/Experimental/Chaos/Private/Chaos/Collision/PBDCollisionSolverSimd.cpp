// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/Collision/PBDCollisionSolverSimd.h"

#include "Chaos/Collision/CollisionApplyType.h"
#include "Chaos/Collision/PBDCollisionConstraint.h"
#include "Chaos/CollisionResolution.h"
#include "Chaos/CollisionResolutionTypes.h"
#include "Chaos/CollisionResolutionUtil.h"
#include "Chaos/Evolution/SolverBodyContainer.h"
#include "Chaos/Particle/ParticleUtilities.h"
#include "Chaos/PBDCollisionConstraints.h"
#include "Chaos/PBDCollisionConstraintsContact.h"
#include "Chaos/Utilities.h"

//PRAGMA_DISABLE_OPTIMIZATION

namespace Chaos
{
	namespace CVars
	{
		extern bool bChaos_PBDCollisionSolver_Velocity_FrictionEnabled;
		extern float Chaos_PBDCollisionSolver_Position_StaticFrictionStiffness;
	}

	namespace Private
	{

		//////////////////////////////////////////////////////////////////////////////////////////////////
		//////////////////////////////////////////////////////////////////////////////////////////////////
		//////////////////////////////////////////////////////////////////////////////////////////////////
		//////////////////////////////////////////////////////////////////////////////////////////////////

		// NOTE: Currently only implemeted for 4-lane SIMD

		// Same as FPlatformMisc::PrefetchBlock() but auto-unrolls the loop and avoids the divide.
		// Somehow this makes things slower
		template<typename T>
		FORCEINLINE static void PrefetchObject(const T* Object)
		{
			constexpr int32 CacheLineSize = PLATFORM_CACHE_LINE_SIZE;
			constexpr int32 NumBytes = sizeof(T);
			constexpr int32 NumLines = (NumBytes + CacheLineSize - 1) / CacheLineSize;

			int32 Offset = 0;
			for (int32 Line = 0; Line < NumLines; ++Line)
			{
				FPlatformMisc::Prefetch(Object, Offset);
				Offset += CacheLineSize;
			}
		}

		// Is there any point in having a value greater than 1? Tune this
		static const int32 PrefetchCount = 4;

		FORCEINLINE void PrefetchSolvePosition(
			const int32 Index,
			const TArrayView<TPBDCollisionSolverSimd<4>>& Solvers,
			const TArrayView<TPBDCollisionSolverManifoldPointsSimd<4>>& ManifoldPoints,
			const TArrayView<TSolverBodyPtrPairSimd<4>>& Bodies)
		{
			if (Index < Solvers.Num())
			{
				FPlatformMisc::PrefetchBlock(&Solvers[Index], sizeof(TPBDCollisionSolverSimd<4>));

				FPlatformMisc::PrefetchBlock(&Solvers[Index].GetManifoldPoint(0, ManifoldPoints), 4 * sizeof(TPBDCollisionSolverManifoldPointsSimd<4>));

				for (int32 LaneIndex = 0; LaneIndex < 4; ++LaneIndex)
				{
					Bodies[Index].Body0.GetValue(LaneIndex)->PrefetchPositionSolverData();
					Bodies[Index].Body1.GetValue(LaneIndex)->PrefetchPositionSolverData();
				}
			}
		}


		FORCEINLINE void PrefetchSolveVelocity(
			const int32 Index,
			const TArrayView<TPBDCollisionSolverSimd<4>>& Solvers,
			const TArrayView<TPBDCollisionSolverManifoldPointsSimd<4>>& ManifoldPoints,
			const TArrayView<TSolverBodyPtrPairSimd<4>>& Bodies)
		{
			if (Index < Solvers.Num())
			{
				FPlatformMisc::PrefetchBlock(&Solvers[Index], sizeof(TPBDCollisionSolverSimd<4>));

				FPlatformMisc::PrefetchBlock(&Solvers[Index].GetManifoldPoint(0, ManifoldPoints), 4 * sizeof(TPBDCollisionSolverManifoldPointsSimd<4>));

				for (int32 LaneIndex = 0; LaneIndex < 4; ++LaneIndex)
				{
					Bodies[Index].Body0.GetValue(LaneIndex)->PrefetchVelocitySolverData();
					Bodies[Index].Body1.GetValue(LaneIndex)->PrefetchVelocitySolverData();
				}
			}
		}

		template<>
		void FPBDCollisionSolverHelperSimd::SolvePositionNoFriction<4>(
			const TArrayView<TPBDCollisionSolverSimd<4>>& Solvers,
			const TArrayView<TPBDCollisionSolverManifoldPointsSimd<4>>& ManifoldPoints,
			const TArrayView<TSolverBodyPtrPairSimd<4>>& Bodies,
			const FSolverReal InDt,
			const FSolverReal InMaxPushOut)
		{
			const FSimd4Realf MaxPushOut = FSimd4Realf::Make(InMaxPushOut);

			for (int32 Index = 0; Index < PrefetchCount; ++Index)
			{
				PrefetchSolvePosition(Index, Solvers, ManifoldPoints, Bodies);
			}

			for (int32 Index = 0; Index < Solvers.Num(); ++Index)
			{
				PrefetchSolvePosition(Index + PrefetchCount, Solvers, ManifoldPoints, Bodies);

				Solvers[Index].SolvePositionNoFriction(ManifoldPoints, Bodies[Index].Body0, Bodies[Index].Body1, MaxPushOut);
			}
		}

		template<>
		void FPBDCollisionSolverHelperSimd::SolvePositionWithFriction<4>(
			const TArrayView<TPBDCollisionSolverSimd<4>>& Solvers,
			const TArrayView<TPBDCollisionSolverManifoldPointsSimd<4>>& ManifoldPoints,
			const TArrayView<TSolverBodyPtrPairSimd<4>>& Bodies,
			const FSolverReal InDt,
			const FSolverReal InMaxPushOut)
		{
			const FSimd4Realf MaxPushOut = FSimd4Realf::Make(InMaxPushOut);
			const FSimd4Realf FrictionStiffnessScale = FSimd4Realf::Make(CVars::Chaos_PBDCollisionSolver_Position_StaticFrictionStiffness);

			for (int32 Index = 0; Index < PrefetchCount; ++Index)
			{
				PrefetchSolvePosition(Index, Solvers, ManifoldPoints, Bodies);
			}

			for (int32 Index = 0; Index < Solvers.Num(); ++Index)
			{
				PrefetchSolvePosition(Index + PrefetchCount, Solvers, ManifoldPoints, Bodies);

				Solvers[Index].SolvePositionWithFriction(ManifoldPoints, Bodies[Index].Body0, Bodies[Index].Body1, MaxPushOut, FrictionStiffnessScale);
			}
		}

		template<>
		void FPBDCollisionSolverHelperSimd::SolveVelocityNoFriction<4>(
			const TArrayView<TPBDCollisionSolverSimd<4>>& Solvers,
			const TArrayView<TPBDCollisionSolverManifoldPointsSimd<4>>& ManifoldPoints,
			const TArrayView<TSolverBodyPtrPairSimd<4>>& Bodies,
			const FSolverReal InDt)
		{
			const FSimd4Realf Dt = FSimd4Realf::Make(InDt);

			for (int32 Index = 0; Index < PrefetchCount; ++Index)
			{
				PrefetchSolveVelocity(Index, Solvers, ManifoldPoints, Bodies);
			}

			for (int32 Index = 0; Index < Solvers.Num(); ++Index)
			{
				PrefetchSolveVelocity(Index + PrefetchCount, Solvers, ManifoldPoints, Bodies);

				Solvers[Index].SolveVelocityNoFriction(ManifoldPoints, Bodies[Index].Body0, Bodies[Index].Body1, Dt);
			}
		}

		template<>
		void FPBDCollisionSolverHelperSimd::SolveVelocityWithFriction<4>(
			const TArrayView<TPBDCollisionSolverSimd<4>>& Solvers,
			const TArrayView<TPBDCollisionSolverManifoldPointsSimd<4>>& ManifoldPoints,
			const TArrayView<TSolverBodyPtrPairSimd<4>>& Bodies,
			const FSolverReal InDt)
		{
			if (!CVars::bChaos_PBDCollisionSolver_Velocity_FrictionEnabled)
			{
				SolveVelocityNoFriction(Solvers, ManifoldPoints, Bodies, InDt);
				return;
			}

			for (int32 Index = 0; Index < PrefetchCount; ++Index)
			{
				PrefetchSolveVelocity(Index, Solvers, ManifoldPoints, Bodies);
			}

			const FSimd4Realf Dt = FSimd4Realf::Make(InDt);
			const FSimd4Realf FrictionStiffnessScale = FSimd4Realf::Make(CVars::Chaos_PBDCollisionSolver_Position_StaticFrictionStiffness);

			for (int32 Index = 0; Index < Solvers.Num(); ++Index)
			{
				PrefetchSolveVelocity(Index + PrefetchCount, Solvers, ManifoldPoints, Bodies);

				Solvers[Index].SolveVelocityWithFriction(ManifoldPoints, Bodies[Index].Body0, Bodies[Index].Body1, Dt, FrictionStiffnessScale);
			}
		}

		void FPBDCollisionSolverHelperSimd::CheckISPC()
		{
		}

	}	// namespace Private
}	// namespace Chaos