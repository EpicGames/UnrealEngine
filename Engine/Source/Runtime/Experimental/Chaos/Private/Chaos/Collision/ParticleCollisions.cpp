// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/Collision/ParticleCollisions.h"
#include "Chaos/Collision/CollisionConstraintAllocator.h"
#include "Chaos/Collision/ParticlePairMidPhase.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/PBDCollisionConstraints.h"

namespace Chaos
{
	FParticleCollisions::FParticleCollisions()
	{
	}

	FParticleCollisions::~FParticleCollisions()
	{
	}

	void FParticleCollisions::AddMidPhase(FGeometryParticleHandle* InParticle, FParticlePairMidPhase* InMidPhase)
	{
		// Double add?
		check(InMidPhase->GetParticleCollisionsIndex(InParticle) == INDEX_NONE);

		const int32 Index = MidPhases.Emplace(typename FContainerType::ElementType(InMidPhase->GetKey().GetKey(), InMidPhase));

		// Set the cookie for the midphase for fast removal
		InMidPhase->SetParticleCollisionsIndex(InParticle, Index);
	}

	void FParticleCollisions::RemoveMidPhase(FGeometryParticleHandle* InParticle, FParticlePairMidPhase* InMidPhase)
	{
		// Retrieve the cookie from the midphase
		const int32 Index = InMidPhase->GetParticleCollisionsIndex(InParticle);

		// Double remove?
		check(Index != INDEX_NONE);
		
		MidPhases.RemoveAtSwap(Index);
		InMidPhase->SetParticleCollisionsIndex(InParticle, INDEX_NONE);

		// Update the cookie of the midphase that moved
		if (Index < MidPhases.Num())
		{
			MidPhases[Index].Value->SetParticleCollisionsIndex(InParticle, Index);
		}
	}

}