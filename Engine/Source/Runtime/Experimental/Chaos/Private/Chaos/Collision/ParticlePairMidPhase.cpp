// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/Collision/ParticlePairMidPhase.h"
#include "Chaos/ChaosDebugDraw.h"
#include "Chaos/Collision/CollisionConstraintAllocator.h"
#include "Chaos/Collision/CollisionContext.h"
#include "Chaos/Collision/CollisionFilter.h"
#include "Chaos/Collision/CollisionUtil.h"
#include "Chaos/Collision/ContactPointsMiscShapes.h"
#include "Chaos/Collision/ContactTriangles.h"
#include "Chaos/Collision/PBDCollisionConstraint.h"
#include "Chaos/CollisionResolution.h"
#include "Chaos/HeightField.h"
#include "Chaos/ImplicitObject.h"
#include "Chaos/ImplicitObjectBVH.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/Particle/ParticleUtilities.h"
#include "Chaos/PBDCollisionConstraints.h"
#include "ChaosStats.h"
#include "Misc/MemStack.h"
#include "Chaos/ConvexOptimizer.h"
#include "ProfilingDebugging/CountersTrace.h"

//UE_DISABLE_OPTIMIZATION

TRACE_DECLARE_INT_COUNTER_EXTERN(ChaosTraceCounter_MidPhase_NumShapePair);
TRACE_DECLARE_INT_COUNTER_EXTERN(ChaosTraceCounter_MidPhase_NumGeneric);

extern bool Chaos_Collision_NarrowPhase_AABBBoundsCheck;
extern bool bChaosCCDResetTOIFix;

// Enable for extended stats. Slow but useful for determining counts and relative costs of midphases
#if 0
#define CHAOS_MIDPHASE_SCOPE_CYCLE_TIMER(X) QUICK_SCOPE_CYCLE_COUNTER(X)
#else
#define CHAOS_MIDPHASE_SCOPE_CYCLE_TIMER(X)
#endif

namespace Chaos
{
#if CHAOS_DEBUG_DRAW
	namespace CVars
	{
		extern int32 ChaosSolverDrawCCDInteractions;
		extern DebugDraw::FChaosDebugDrawSettings ChaosSolverDebugDebugDrawSettings;
	}
	namespace DebugDraw
	{
		extern float ChaosDebugDrawCCDDuration;
	}
#endif

	namespace CVars
	{
		FRealSingle Chaos_Collision_CullDistanceScaleInverseSize = 0.01f;	// 100cm
		FRealSingle Chaos_Collision_MinCullDistanceScale = 1.0f;
		FAutoConsoleVariableRef CVarChaos_Collision_CullDistanceReferenceSize(TEXT("p.Chaos.Collision.CullDistanceReferenceSize"), Chaos_Collision_CullDistanceScaleInverseSize, TEXT(""));
		FAutoConsoleVariableRef CVarChaos_Collision_MinCullDistanecScale(TEXT("p.Chaos.Collision.MinCullDistanceScale"), Chaos_Collision_MinCullDistanceScale, TEXT(""));

		// Whether we support the pre-flattened shape pair list which optimizes the common midphase case of collision between particles 
		// with only a small number implicit objects each. If so, how many shape pairs do we allow before switching to the generic version?
		bool bChaos_Collision_MidPhase_EnableShapePairs = true;
		int32 Chaos_Collision_MidPhase_MaxShapePairs = 100;
		FAutoConsoleVariableRef CVarChaos_Collision_EnableShapePairs(TEXT("p.Chaos.Collision.EnableShapePairs"), bChaos_Collision_MidPhase_EnableShapePairs, TEXT(""));
		FAutoConsoleVariableRef CVarChaos_Collision_MaxShapePairs(TEXT("p.Chaos.Collision.MaxShapePairs"), Chaos_Collision_MidPhase_MaxShapePairs, TEXT(""));

		// This should be enabled but it exposes a bug that must be fixed first: collisions with 
		// invalid particle pointers are being activated causing a crash in CCDUtilities::ApplyCorrections
		bool bChaosMidPhaseActivateWakingConstraints = true;
		FAutoConsoleVariableRef CVarChaosMidPhaseActivateWakingConstraints(TEXT("p.Chaos.Bugs.ActivateWakingConstraints"), bChaosMidPhaseActivateWakingConstraints, TEXT(""));

		// See FGenericParticlePairMidPhase::SetConvexOptimizationIsActive
		bool bChaosMidPhaseModifierConvexOptimizerFix = true;
		FAutoConsoleVariableRef CVarChaosMidPhaseModifierConvexOptimizerFix(TEXT("p.Chaos.Bugs.MidPhaseModifierConvexOptimizerFix"), bChaosMidPhaseModifierConvexOptimizerFix, TEXT(""));

		FRealSingle ChaosOneWayPairSphereRadiusAlpha = 1.0f;
		FAutoConsoleVariableRef CVarChaosOneWayPairSphereRadiusAlpha(TEXT("p.Chaos.Solver.OneWayPairSphereRadiusAlpha"), ChaosOneWayPairSphereRadiusAlpha, TEXT("If one-way interactions are treated as spheres, the diameter is lerp(Size.Min, Size.Max, RadiusAlpha)"));

		extern int32 ChaosOneWayInteractionPairCollisionMode;

		extern bool bChaosForceMACD;
	}

	using namespace CVars;


	////////////////////////////////////////////////////////////////////////////////////////
	////////////////////////////////////////////////////////////////////////////////////////
	////////////////////////////////////////////////////////////////////////////////////////
	////////////////////////////////////////////////////////////////////////////////////////

	inline bool ImplicitOverlapOBBToAABB(
		const FImplicitObject* Implicit0,
		const FImplicitObject* Implicit1,
		const FRigidTransform3& ShapeTransform1To0,
		const FVec3f& LocalRelativeMovement0,
		const FReal CullDistance)
	{
		if (Implicit0->HasBoundingBox() && Implicit1->HasBoundingBox())
		{
			const FAABB3 Box1In0 = Implicit1->CalculateTransformedBounds(ShapeTransform1To0).GrowByVector(LocalRelativeMovement0).Thicken(CullDistance);
			const FAABB3 Box0 = Implicit0->BoundingBox();
			return Box0.Intersects(Box1In0);
		}
		return true;
	}

	inline bool ImplicitOverlapOBBToAABB(
		const FImplicitObject* Implicit0,
		const FImplicitObject* Implicit1,
		const FRigidTransform3& ShapeWorldTransform0,
		const FRigidTransform3& ShapeWorldTransform1,
		const FVec3f& RelativeMovement,
		const FReal CullDistance)
	{
		const FRigidTransform3 ShapeTransform1To0 = ShapeWorldTransform1.GetRelativeTransform(ShapeWorldTransform0);
		const FVec3 LocalRelativeMovement0 = ShapeWorldTransform0.InverseTransformVectorNoScale(FVec3(RelativeMovement));
		return ImplicitOverlapOBBToAABB(Implicit0, Implicit1, ShapeTransform1To0, LocalRelativeMovement0, CullDistance);
	}

	// Get the number of leaf objects in the implicit hierarchy, and set a flag if this hierarchy is a tree
	// (i.e., contains a Union of Unions ro a BVH, rather than a single flat union at the root)
	inline int32 GetNumLeafImplicits(const FImplicitObject* Implicit)
	{
		int32 NumImplicits = 0;

		if (Implicit != nullptr)
		{
			// All implicit excepts unions have 1 leaf
			NumImplicits = 1;

			if (const FImplicitObjectUnion* Union = Implicit->template AsA<FImplicitObjectUnion>())
			{
				NumImplicits = Union->GetNumLeafObjects();
			}
		}

		return NumImplicits;
	}

	// Get the ShapeInstance data from the particle for the implicit with the specified root object index (its index
	// in the root union implicit if there is one). Usually every implicit in the root union is represented in the
	// ShapesArray, but not always. GCS and ClusterUnions sometimes have a Union at the root but only a single ShapeInstance
	const FShapeInstance* GetShapeInstance(const FShapeInstanceArray& ShapeInstances, const int32 RootObjectIndex, const Private::FConvexOptimizer* ConvexOptimizer = nullptr)
	{
		if(ConvexOptimizer && ConvexOptimizer->IsValid() && (RootObjectIndex == INDEX_NONE))
		{
			return ConvexOptimizer->GetShapeInstances()[0].Get();
		}
		else
		{
			const int32 ShapeIndex = (ShapeInstances.IsValidIndex(RootObjectIndex)) ? RootObjectIndex : 0;
			return ShapeInstances[ShapeIndex].Get();
		}
	}


	// Find the LevelSet particles for an ImplicitObject. This used to just be the BVHParticles on the owning Particle,
	// but we can now merge particles with BVHs into a ClusterUnion sp we can end up with LevelSet implicits referencing
	// BVHParticles from other sources. Therefore We need to search the hierarchy for the implicit and if there is an 
	// ImplicitObjectUnionClustered in the hierarchy, it will contain a mapping to the original particle so we can get 
	// its BVHParticles.
	// @todo(chaos): we should rework how BVHParticles are created and stored and fix this
	const FBVHParticles* FindLevelSetParticles(const FGeometryParticleHandle* Particle, const FImplicitObject* Implicit)
	{
		const FBVHParticles* BVHParticles = nullptr;

		if (const FPBDRigidParticleHandle* Rigid = Particle->CastToRigidParticle())
		{
			// Visit the hierarchy in pre-order depth-first traversal. If we hit a UnionClustered, see if it has our particles
			Implicit->VisitObjects(
				[Implicit, &BVHParticles](const FImplicitObject* HierarchyImplicit, const FRigidTransform3& Transform, const int32 RootObjectIndex, const int32 ObjectIndex, const int32 LeafObjectIndex) -> bool
				{
					if (const FImplicitObjectUnionClustered* UnionClustered = HierarchyImplicit->template GetObject<FImplicitObjectUnionClustered>())
					{
						// If this union has our implicit in its map, extract the collision particles and we're done
						if (const FPBDRigidParticleHandle* SourceRigid = UnionClustered->FindParticleForImplicitObject(Implicit))
						{
							BVHParticles = SourceRigid->CollisionParticles().Get();
							return false;
						}
					}

					// This is not the implicit we're looking for. Move along.
					return true;
				});

			// If we didn't find a source particle, it must be ours (if we have any)
			if (BVHParticles == nullptr)
			{
				BVHParticles = Rigid->CollisionParticles().Get();
			}
		}

		return BVHParticles;
	}

	FPBDCollisionConstraintPtr CreateShapePairConstraint(
		FGeometryParticleHandle* Particle0,
		const FPerShapeData* InShape0,
		FGeometryParticleHandle* Particle1,
		const FPerShapeData* InShape1,
		const FReal CullDistance,
		const EContactShapesType ShapePairType,
		const bool bUseManifold,
		const FCollisionContext& Context)
	{
		const FImplicitObject* Implicit0 = InShape0->GetLeafGeometry();
		const FBVHParticles* BVHParticles0 = FConstGenericParticleHandle(Particle0)->CollisionParticles().Get();
		const FRigidTransform3& ShapeRelativeTransform0 = (FRigidTransform3)InShape0->GetLeafRelativeTransform();
		const FImplicitObject* Implicit1 = InShape1->GetLeafGeometry();
		const FBVHParticles* BVHParticles1 = FConstGenericParticleHandle(Particle1)->CollisionParticles().Get();
		const FRigidTransform3& ShapeRelativeTransform1 = (FRigidTransform3)InShape1->GetLeafRelativeTransform();

		return Context.GetAllocator()->CreateConstraint(Particle0, Implicit0, InShape0, BVHParticles0, ShapeRelativeTransform0, Particle1, Implicit1, InShape1, BVHParticles1, ShapeRelativeTransform1, CullDistance, bUseManifold, ShapePairType);
	}

	FPBDCollisionConstraintPtr CreateImplicitPairConstraint(
		FGeometryParticleHandle* Particle0,
		const FImplicitObject* Implicit0,
		const FPerShapeData* Shape0,
		const FBVHParticles* BVHParticles0,
		const FRigidTransform3& ShapeRelativeTransform0,
		FGeometryParticleHandle* Particle1,
		const FImplicitObject* Implicit1,
		const FPerShapeData* Shape1,
		const FBVHParticles* BVHParticles1,
		const FRigidTransform3& ShapeRelativeTransform1,
		const FReal CullDistance,
		const EContactShapesType ShapePairType,
		const bool bUseManifold,
		const FCollisionContext& Context)
	{
		return Context.GetAllocator()->CreateConstraint(Particle0, Implicit0, Shape0, BVHParticles0, ShapeRelativeTransform0, Particle1, Implicit1, Shape1, BVHParticles1, ShapeRelativeTransform1, CullDistance, bUseManifold, ShapePairType);
	}

	// A unique key for a collision between two particles (key is only unique within the particle pair midphase)
	class FParticlePairMidPhaseCollisionKey
	{
	public:

		FParticlePairMidPhaseCollisionKey()
			: Key(0)
		{
		}

		FParticlePairMidPhaseCollisionKey(const int32 InShapeID0, const int32 InShapeID1)
		{
			Generate(InShapeID0, InShapeID1);
		}

		uint64 GetKey() const
		{
			return Key;
		}

		friend bool operator==(const FParticlePairMidPhaseCollisionKey& L, const FParticlePairMidPhaseCollisionKey& R)
		{
			return L.Key == R.Key;
		}

		friend bool operator!=(const FParticlePairMidPhaseCollisionKey& L, const FParticlePairMidPhaseCollisionKey& R)
		{
			return !(L == R);
		}

		friend bool operator<(const FParticlePairMidPhaseCollisionKey& L, const FParticlePairMidPhaseCollisionKey& R)
		{
			return L.Key < R.Key;
		}

	private:
		void Generate(const int32 InShapeID0, const int32 InShapeID1)
		{
			ShapeID0 = InShapeID0;
			ShapeID1 = InShapeID1;
		}

		union {
			struct
			{
				int32 ShapeID0;
				int32 ShapeID1;
			};
			uint64 Key;
		};
	};

	void ReactivateCollisionConstraint(FPBDCollisionConstraint* Constraint)
	{
		// Update the epoch and add to the active list
		Constraint->ConcreteContainer()->GetConstraintAllocator().AddActiveConstraint(Constraint);

		// Re-enable the constraint
		// NOTE: The Enabled/Disabled flag can also be controlled by the user in a
		// contact modifier. There is no danger of re-enabling a constraint that the 
		// user disabled because a disabled constraint is removed from the graph and 
		// therefore will not get reactivated if an island wakes.
		// @todo(chaos): Despite the note, we should separate Internal and Extenal Enabled flags.
		// Internal-Enabled would be controlled by collision detection and waking. 
		// External-Enabled would be controlled by the user via a contact callback.
		// We would only be enabled when both are set. Probably call them Activated and Enabled.
		Constraint->SetEnabled(true);
	}


	////////////////////////////////////////////////////////////////////////////////////////
	////////////////////////////////////////////////////////////////////////////////////////
	////////////////////////////////////////////////////////////////////////////////////////
	////////////////////////////////////////////////////////////////////////////////////////


	FSingleShapePairCollisionDetector::FSingleShapePairCollisionDetector(
		FGeometryParticleHandle* InParticle0,
		const FPerShapeData* InShape0,
		FGeometryParticleHandle* InParticle1,
		const FPerShapeData* InShape1,
		const Private::FCollisionSortKey& InCollisionSortKey,
		const EContactShapesType InShapePairType,
		FParticlePairMidPhase& InMidPhase)
		: MidPhase(InMidPhase)
		, Constraint(nullptr)
		, Particle0(InParticle0)
		, Particle1(InParticle1)
		, Shape0(InShape0)
		, Shape1(InShape1)
		, CollisionSortKey(InCollisionSortKey)
		, SphereBoundsCheckSize(0)
		, LastUsedEpoch(-1)
		, ShapePairType(InShapePairType)
		, BoundsTestFlags()
	{
		const FImplicitObject* Implicit0 = InShape0->GetLeafGeometry();
		const FImplicitObject* Implicit1 = InShape1->GetLeafGeometry();

		BoundsTestFlags = Private::CalculateImplicitBoundsTestFlags(
			InParticle0, Implicit0, InShape0,
			InParticle1, Implicit1, InShape1,
			SphereBoundsCheckSize);
	}

	FSingleShapePairCollisionDetector::FSingleShapePairCollisionDetector(FSingleShapePairCollisionDetector&& R)
		: MidPhase(R.MidPhase)
		, Constraint(MoveTemp(R.Constraint))
		, Particle0(R.Particle0)
		, Particle1(R.Particle1)
		, Shape0(R.Shape0)
		, Shape1(R.Shape1)
		, CollisionSortKey(R.CollisionSortKey)
		, SphereBoundsCheckSize(R.SphereBoundsCheckSize)
		, ShapePairType(R.ShapePairType)
		, BoundsTestFlags(R.BoundsTestFlags)
	{
	}

	FSingleShapePairCollisionDetector::~FSingleShapePairCollisionDetector()
	{
	}

	bool FSingleShapePairCollisionDetector::DoBoundsOverlap(
		const FRealSingle CullDistance, 
		const FVec3f& RelativeMovement, 
		const int32 CurrentEpoch)
	{
		PHYSICS_CSV_SCOPED_EXPENSIVE(PhysicsVerbose, NarrowPhase_ShapeBounds);

		const FAABB3& ShapeWorldBounds0 = Shape0->GetWorldSpaceShapeBounds();
		const FAABB3& ShapeWorldBounds1 = Shape1->GetWorldSpaceShapeBounds();

		// World-space expanded bounds check
		if (BoundsTestFlags.bEnableAABBCheck)
		{
			const FAABB3 ExpandedShapeWorldBounds0 = FAABB3(ShapeWorldBounds0).GrowByVector(-RelativeMovement).Thicken(CullDistance);
			if (!ExpandedShapeWorldBounds0.Intersects(ShapeWorldBounds1))
			{
				return false;
			}
		}

		// World-space sphere bounds check
		if (BoundsTestFlags.bEnableDistanceCheck && (SphereBoundsCheckSize > FRealSingle(0)))
		{
			const FVec3 Separation = ShapeWorldBounds0.GetCenter() - ShapeWorldBounds1.GetCenter();
			const FReal SeparationSq = Separation.SizeSquared();
			const FReal CullDistanceSq = FMath::Square(CullDistance + FReal(SphereBoundsCheckSize));
			if (SeparationSq > CullDistanceSq)
			{
				return false;
			}
		}

		// OBB-AABB test on both directions. This is beneficial for shapes which do not fit their AABBs very well,
		// which includes boxes and other shapes that are not roughly spherical. It is especially beneficial when
		// one shape is long and thin (i.e., it does not fit an AABB well when the shape is rotated).
		// However, it is quite expensive to do this all the time so we only do this test when we did not 
		// collide last frame. This is ok if we assume not much changes from frame to frame, but it means
		// we might call the narrow phase one time too many when shapes become separated.
		const int32 LastEpoch = CurrentEpoch - 1;
		const bool bCollidedLastTick = IsUsedSince(LastEpoch);
		if ((BoundsTestFlags.bEnableOBBCheck0 || BoundsTestFlags.bEnableOBBCheck1) && !bCollidedLastTick)
		{
			const FRigidTransform3& ShapeWorldTransform0 = Shape0->GetLeafWorldTransform(GetParticle0());
			const FRigidTransform3& ShapeWorldTransform1 = Shape1->GetLeafWorldTransform(GetParticle1());
			const FImplicitObject* Implicit0 = Shape0->GetLeafGeometry();
			const FImplicitObject* Implicit1 = Shape1->GetLeafGeometry();

			if (BoundsTestFlags.bEnableOBBCheck0)
			{
				if (!ImplicitOverlapOBBToAABB(Implicit0, Implicit1, ShapeWorldTransform0, ShapeWorldTransform1, RelativeMovement, CullDistance))
				{
					return false;
				}
			}

			if (BoundsTestFlags.bEnableOBBCheck1)
			{
				if (!ImplicitOverlapOBBToAABB(Implicit1, Implicit0, ShapeWorldTransform1, ShapeWorldTransform0, -RelativeMovement, CullDistance))
				{
					return false;
				}
			}
		}

		return true;
	}

	int32 FSingleShapePairCollisionDetector::GenerateCollision(
		const FRealSingle Dt,
		const FRealSingle CullDistance,
		const FVec3f& RelativeMovement,
		const FCollisionContext& Context)
	{
		CHAOS_MIDPHASE_SCOPE_CYCLE_TIMER(FSingleShapePairCollisionDetector_GenerateCollision);

		const int32 CurrentEpoch = Context.GetAllocator()->GetCurrentEpoch();
		if (DoBoundsOverlap(CullDistance, RelativeMovement, CurrentEpoch))
		{
			return GenerateCollisionImpl(Dt, CullDistance, RelativeMovement, Context);
		}
		return 0;
	}

	int32 FSingleShapePairCollisionDetector::GenerateCollisionCCD(
		const FRealSingle Dt,
		const FRealSingle CullDistance,
		const FVec3f& RelativeMovement,
		const bool bEnableCCDSweep,
		const FCollisionContext& Context)
	{
		CHAOS_MIDPHASE_SCOPE_CYCLE_TIMER(FSingleShapePairCollisionDetector_GenerateCollisionCCD);

		return GenerateCollisionCCDImpl(Dt, CullDistance, RelativeMovement, bEnableCCDSweep, Context);
	}

	void FSingleShapePairCollisionDetector::CreateConstraint(const FReal CullDistance, const FCollisionContext& Context)
	{
		PHYSICS_CSV_SCOPED_EXPENSIVE(PhysicsVerbose, NarrowPhase_CreateConstraint);
		check(!Constraint.IsValid());

		Constraint = CreateShapePairConstraint(GetParticle0(), Shape0, GetParticle1(), Shape1, CullDistance, ShapePairType, Context.GetSettings().bAllowManifolds, Context);

		const int32 CurrentEpoch = Context.GetAllocator()->GetCurrentEpoch();
		Constraint->GetContainerCookie().MidPhase = &MidPhase;
		Constraint->GetContainerCookie().bIsMultiShapePair = false;
		Constraint->GetContainerCookie().CreationEpoch = CurrentEpoch;

		Constraint->SetCollisionSortKey(CollisionSortKey);

		LastUsedEpoch = -1;
	}

	int32 FSingleShapePairCollisionDetector::GenerateCollisionImpl(
		const FRealSingle Dt,
		const FRealSingle CullDistance,
		const FVec3f& RelativeMovement,
		const FCollisionContext& Context)
	{
		CHAOS_MIDPHASE_SCOPE_CYCLE_TIMER(FSingleShapePairCollisionDetector_GenerateCollisionImpl);

		if (BoundsTestFlags.bIsProbe)
		{
			return GenerateCollisionProbeImpl(Dt, CullDistance, RelativeMovement, Context);
		}

		if (!Constraint.IsValid())
		{
			// Lazy creation of the constraint. If a shape pair never gets within CullDistance of each
			// other, we never allocate a constraint for them. Once they overlap, we reuse the constraint
			// until the owing particles are not overlapping. i.e., we keep the constraint even if
			// the shape pairs stop overlapping, reusing it if they start overlapping again.
			CreateConstraint(CullDistance, Context);
		}

		if (Constraint.IsValid())
		{
			PHYSICS_CSV_SCOPED_EXPENSIVE(PhysicsVerbose, NarrowPhase_UpdateConstraint);

			const FRigidTransform3 ShapeWorldTransform0 = Shape0->GetLeafWorldTransform(GetParticle0());
			const FRigidTransform3 ShapeWorldTransform1 = Shape1->GetLeafWorldTransform(GetParticle1());
			const int32 CurrentEpoch = Context.GetAllocator()->GetCurrentEpoch();
			const int32 LastEpoch = CurrentEpoch - 1;
			const bool bWasUpdatedLastTick = IsUsedSince(LastEpoch);

			// Update the world shape transforms on the constraint (we cannot just give it the PerShapeData 
			// pointer because of Unions - see FMultiShapePairCollisionDetector)
			Constraint->SetShapeWorldTransforms(ShapeWorldTransform0, ShapeWorldTransform1);

			Constraint->SetCullDistance(CullDistance);
			Constraint->SetRelativeMovement(RelativeMovement);

			// Constraint may have been previously used with CCD enabled (e.g., a midphase modifier)
			// so we need to make sure that the CCD flag is disabled
			Constraint->SetCCDEnabled(false);
			if (bChaosCCDResetTOIFix)
			{
				Constraint->ResetCCDTimeOfImpact();
			}

			// If the constraint was not used last frame, it needs to be reset, otherwise we will try to reuse
			if (!bWasUpdatedLastTick || (Constraint->GetManifoldPoints().Num() == 0))
			{
				// Clear all manifold data including saved contact data
				Constraint->ResetManifold();
			}
			
			bool bWasManifoldRestored = false;
			const bool bAllowManifoldRestore = Context.GetSettings().bAllowManifoldReuse && BoundsTestFlags.bEnableManifoldUpdate;
			if (bAllowManifoldRestore && bWasUpdatedLastTick && Constraint->GetCanRestoreManifold())
			{
				// Update the existing manifold. We can re-use as-is if none of the points have moved much and the bodies have not moved much
				// NOTE: this can succeed in "restoring" even if we have no manifold points
				// NOTE: this uses the transforms from SetLastShapeWorldTransforms, so we can only do this if we were updated last tick
				bWasManifoldRestored = Constraint->TryRestoreManifold();
			}

			if (!bWasManifoldRestored)
			{
				// We are not trying to (or chose not to) reuse manifold points, so reset them but leave stored data intact (for friction)
				Constraint->ResetActiveManifoldContacts();

				if (!Context.GetSettings().bDeferNarrowPhase)
				{
					CHAOS_MIDPHASE_SCOPE_CYCLE_TIMER(FSingleShapePairCollisionDetector_GenerateCollision_NarrowPhase);

					// Run the narrow phase
					Collisions::UpdateConstraint(*Constraint.Get(), ShapeWorldTransform0, ShapeWorldTransform1, Dt);
				}

				// We will be updating the manifold so update transforms used to check for movement in UpdateAndTryRestoreManifold on future ticks
				// NOTE: We call this after Collisions::UpdateConstraint because it may reset the manifold and reset the bCanRestoreManifold flag.
				// @todo(chaos): Collisions::UpdateConstraint does not need to reset the manifold - fix that
				Constraint->SetLastShapeWorldTransforms(ShapeWorldTransform0, ShapeWorldTransform1);
			}

			// If we have a valid contact, add it to the active list
			// We also add it to the active list if collision detection is deferred because the data will be filled in later and we
			// don't know in advance whether we will pass the Phi check (deferred narrow phase is used with RBAN)
			if (Constraint->GetPhi() <= CullDistance || Context.GetSettings().bDeferNarrowPhase)
			{
				Constraint->SetIsInitialContact(!bWasUpdatedLastTick);

				if (Context.GetAllocator()->ActivateConstraint(Constraint.Get()))
				{
					LastUsedEpoch = CurrentEpoch;
					return 1;
				}
			}
		}

		return 0;
	}

	int32 FSingleShapePairCollisionDetector::GenerateCollisionCCDImpl(
		const FRealSingle Dt,
		const FRealSingle CullDistance,
		const FVec3f& RelativeMovement,
		const bool bEnableCCDSweep,
		const FCollisionContext& Context)
	{
		if (BoundsTestFlags.bIsProbe)
		{
			return GenerateCollisionProbeImpl(Dt, CullDistance, RelativeMovement, Context);
		}

		if (!Constraint.IsValid())
		{
			// Lazy creation of the constraint. 
			CreateConstraint(CullDistance, Context);
		}

		// Constraint may have been previously used with CCD disabled (e.g., a midphase modifier)
		// so we need to make sure that the CCD flags are set appropriately
		if (Constraint.IsValid())
		{
			check(!Constraint->IsEnabled());
			Constraint->SetCCDEnabled(true);
			Constraint->SetCCDSweepEnabled(bEnableCCDSweep);
			if (bChaosCCDResetTOIFix)
			{
				Constraint->ResetCCDTimeOfImpact();
			}
		}

		if (!bEnableCCDSweep)
		{
			return GenerateCollision(Dt, CullDistance, RelativeMovement, Context);
		}

		// Swept collision detection
		if (Constraint.IsValid())
		{
			PHYSICS_CSV_SCOPED_EXPENSIVE(PhysicsVerbose, NarrowPhase_UpdateConstraintCCD);

			const FRigidTransform3& ShapeWorldTransform0 = Shape0->GetLeafWorldTransform(GetParticle0());
			const FRigidTransform3& ShapeWorldTransform1 = Shape1->GetLeafWorldTransform(GetParticle1());

			// Update the world shape transforms on the constraint (we cannot just give it the PerShapeData 
			// pointer because of Unions - see FMultiShapePairCollisionDetector)
			// NOTE: these are not used by CCD which continuously moves the particles
			Constraint->SetShapeWorldTransforms(ShapeWorldTransform0, ShapeWorldTransform1);

			Constraint->SetCullDistance(CullDistance);
			Constraint->ResetManifold();
			Constraint->ResetActiveManifoldContacts();

			FConstGenericParticleHandle P0 = GetParticle0();
			FConstGenericParticleHandle P1 = GetParticle1();


			// We need the previous transform for the swept collision detector. It assumes that the current
			// transform has been set on the constraint. 
			// We assume that the particle's center of mass moved in a straight line and that it's rotation has 
			// not changed so we calculate sthe previous transform from the current one and the velocity.
			// NOTE: These are actor transforms, not CoM transforms
			// @todo(chaos): Pass both start and end transforms to the collision detector
			const FRigidTransform3 CCDParticleWorldTransform0 = FRigidTransform3(P0->P() - P0->V() * Dt, P0->Q());
			const FRigidTransform3 CCDParticleWorldTransform1 = FRigidTransform3(P1->P() - P1->V() * Dt, P1->Q());
			const FRigidTransform3 CCDShapeWorldTransform0 = Constraint->GetShapeRelativeTransform0() * CCDParticleWorldTransform0;
			const FRigidTransform3 CCDShapeWorldTransform1 = Constraint->GetShapeRelativeTransform1() * CCDParticleWorldTransform1;
			const bool bDidSweep = Collisions::UpdateConstraintSwept(*Constraint.Get(), CCDShapeWorldTransform0, CCDShapeWorldTransform1, Dt);

#if CHAOS_DEBUG_DRAW
			if (CVars::ChaosSolverDrawCCDInteractions)
			{
				if (FConstGenericParticleHandle(Constraint->GetParticle0())->CCDEnabled())
				{
					DebugDraw::DrawShape(CCDShapeWorldTransform0, Constraint->GetImplicit0(), Shape0, FColor::Black, DebugDraw::ChaosDebugDrawCCDDuration, &CVars::ChaosSolverDebugDebugDrawSettings);
					DebugDraw::DrawShape(ShapeWorldTransform0, Constraint->GetImplicit0(), Shape0, FColor::White, DebugDraw::ChaosDebugDrawCCDDuration, &CVars::ChaosSolverDebugDebugDrawSettings);
				}
				if (FConstGenericParticleHandle(Constraint->GetParticle1())->CCDEnabled())
				{
					DebugDraw::DrawShape(CCDShapeWorldTransform1, Constraint->GetImplicit1(), Shape1, FColor::Black, DebugDraw::ChaosDebugDrawCCDDuration, &CVars::ChaosSolverDebugDebugDrawSettings);
					DebugDraw::DrawShape(ShapeWorldTransform1, Constraint->GetImplicit1(), Shape1, FColor::White, DebugDraw::ChaosDebugDrawCCDDuration, &CVars::ChaosSolverDebugDebugDrawSettings);
				}
			}
#endif

			// @todo(chaos): we always activate swept constraints because the CCD roll-back loops over the active constraints.
			// Fix this - we should only activate constraints when we have a Phi within CullDistance
			bool bShouldActivate = bDidSweep && (Constraint->GetCCDTimeOfImpact() < FReal(1));

			// If we did not get a sweep hit (TOI > 1) or did not sweep (bDidSweep = false), we need to run standard collision detection at T=1.
			// Likewise, if we did get a sweep hit but it's at TOI = 1, treat this constraint as a regular non-swept constraint and skip the rewind.
			// NOTE: The sweep will report TOI==1 for "shallow" sweep hits below the CCD thresholds in the constraint.
			if ((!bDidSweep) || (Constraint->GetCCDTimeOfImpact() >= FReal(1)))
			{
				// @todo(chaos): should we use a reduced cull distance if we get here? The cull distance will have been set based on movement speed...
				Collisions::UpdateConstraint(*Constraint.Get(), Constraint->GetShapeWorldTransform0(), Constraint->GetShapeWorldTransform1(), Dt);
				Constraint->SetCCDSweepEnabled(false);
				bShouldActivate = (Constraint->GetPhi() <= CullDistance);
			}

			if (bShouldActivate)
			{
				const int32 CurrentEpoch = Context.GetAllocator()->GetCurrentEpoch();
				const int32 LastEpoch = CurrentEpoch - 1;
				const bool bWasUpdatedLastTick = IsUsedSince(LastEpoch);
				Constraint->SetIsInitialContact(!bWasUpdatedLastTick);

				if (Context.GetAllocator()->ActivateConstraint(Constraint.Get()))
				{
					LastUsedEpoch = CurrentEpoch;
				}
			}

			return 1;
		}

		return 0;
	}

	int32 FSingleShapePairCollisionDetector::GenerateCollisionProbeImpl(
		const FRealSingle Dt,
		const FRealSingle CullDistance,
		const FVec3f& RelativeMovement,
		const FCollisionContext& Context)
	{
		// Same as regular constraint generation, but always defer narrow phase.
		// Don't do any initial constraint computations.

		if (!Constraint.IsValid())
		{
			CreateConstraint(CullDistance, Context);
		}
		check(!Constraint->IsEnabled());

		if (Constraint.IsValid())
		{
			PHYSICS_CSV_SCOPED_EXPENSIVE(PhysicsVerbose, NarrowPhase_UpdateConstraintProbe);

			Context.GetAllocator()->ActivateConstraint(Constraint.Get());
			LastUsedEpoch = Context.GetAllocator()->GetCurrentEpoch();
			return 1;
		}

		return 0;
	}

	void FSingleShapePairCollisionDetector::WakeCollision(const int32 SleepEpoch, const int32 CurrentEpoch)
	{
		if (Constraint.IsValid() && IsUsedSince(SleepEpoch))
		{
			if (bChaosMidPhaseActivateWakingConstraints)
			{
				ReactivateCollisionConstraint(Constraint.Get());
				LastUsedEpoch = Constraint->GetContainerCookie().LastUsedEpoch;
			}
			else
			{
				// We need to refresh the epoch so that the constraint state will be used as the previous
				// state if the pair is still colliding in the next tick. However, we are not in the active 
				// collisions array so reset the indices.
				Constraint->GetContainerCookie().LastUsedEpoch = CurrentEpoch;
				Constraint->GetContainerCookie().ConstraintIndex = INDEX_NONE;
				Constraint->GetContainerCookie().CCDConstraintIndex = INDEX_NONE;
				LastUsedEpoch = CurrentEpoch;
			}

			// We have skipped collision detection for this particle because it was asleep, so we need to update the transforms...
			// NOTE: this relies on the shape world transforms being up-to-date. They are usually updated in Integrate which
			// is also skipped for sleeping particles, so they must be updated manually when waking partciles (see IslandManager)
			Constraint->SetShapeWorldTransforms(Shape0->GetLeafWorldTransform(GetParticle0()), Shape1->GetLeafWorldTransform(GetParticle1()));
		}
	}

	// This function is called by the resim system to add/replace a collision with one from the history buffer
	void FSingleShapePairCollisionDetector::SetCollision(const FPBDCollisionConstraint& SourceConstraint, const FCollisionContext& Context)
	{
		const int32 CurrentEpoch = Context.GetAllocator()->GetCurrentEpoch();

		if (!Constraint.IsValid())
		{
			Constraint = Context.GetAllocator()->CreateConstraint();
			Constraint->GetContainerCookie().MidPhase = &MidPhase;
			Constraint->GetContainerCookie().bIsMultiShapePair = false;
			Constraint->GetContainerCookie().CreationEpoch = CurrentEpoch;
		}

		// Copy the constraint data over the existing one (ensure we do not replace data required by the graph and the allocator/container)
		Constraint->RestoreFrom(SourceConstraint);

		// Add the constraint to the active list
		// If the constraint already existed and was already active, this will do nothing
		Context.GetAllocator()->ActivateConstraint(Constraint.Get());
		LastUsedEpoch = CurrentEpoch;
	}


	////////////////////////////////////////////////////////////////////////////////////////
	////////////////////////////////////////////////////////////////////////////////////////
	////////////////////////////////////////////////////////////////////////////////////////
	////////////////////////////////////////////////////////////////////////////////////////


	FParticlePairMidPhase::FParticlePairMidPhase(const EParticlePairMidPhaseType InMidPhaseType)
		: MidPhaseType(InMidPhaseType)
		, Flags()
		, Particle0(nullptr)
		, Particle1(nullptr)
		, CullDistanceScale(1)
		, ParticlePairKey()
		, LastUsedEpoch(INDEX_NONE)
		, NumActiveConstraints(0)
		, ParticleCollisionsIndex0(INDEX_NONE)
		, ParticleCollisionsIndex1(INDEX_NONE)
	{
	}

	FParticlePairMidPhase::~FParticlePairMidPhase()
	{
	}

	EParticlePairMidPhaseType FParticlePairMidPhase::CalculateMidPhaseType(FGeometryParticleHandle* InParticle0, FGeometryParticleHandle* InParticle1)
	{
		// For particles with a small number of shapes, we prebuild the set of potentially colliding
		// shape pairs and cache various data with them. However we don't do this if there are a
		// large number of shapes, or if the geometry hierarchy contains Unions of Unions.
		//
		// @todo(chaos): we should be able to support Unions of Unions in the accelerated path
		// when there aren't many objects (with some changes to FSingleShapePairCollisionDetector)

		// We can force use of the Generic midphase (mainly for testing)
		if (!CVars::bChaos_Collision_MidPhase_EnableShapePairs)
		{
			return EParticlePairMidPhaseType::Generic;
		}

		// How many implicits does each particle have?
		const int32 NumImplicits0 = GetNumLeafImplicits(InParticle0->GetGeometry());
		const int32 NumImplicits1 = GetNumLeafImplicits(InParticle1->GetGeometry());

		// Do we have a ShapeInstance for every implicit object?
		// Only the implicits in the root union are represented in the shapes array
		const bool bIsTree0 = (NumImplicits0 > InParticle0->ShapeInstances().Num());
		const bool bIsTree1 = (NumImplicits1 > InParticle1->ShapeInstances().Num());

		// Are there too many implicits for the flattened shape pair path to be optimal?
		const int32 MaxImplicitPairs = CVars::Chaos_Collision_MidPhase_MaxShapePairs;
		const bool bTooManyImplicitPairs = (MaxImplicitPairs > 0) && (NumImplicits0 * NumImplicits1 > MaxImplicitPairs);

		// Is either of the implicits a Union of Unions?
		const bool bIsTree = bIsTree0 || bIsTree1;
	
		const bool bCanPrebuildShapePairs = !bTooManyImplicitPairs && !bIsTree;

		// If both particles have one-way interaction enabled, we might want to treat them as spheres
		if (bCanPrebuildShapePairs && (CVars::ChaosOneWayInteractionPairCollisionMode == (int32)EOneWayInteractionPairCollisionMode::SphereCollision))
		{
			const bool bIsOneWay0 = FConstGenericParticleHandle(InParticle0)->OneWayInteraction();
			const bool bIsOneWay1 = FConstGenericParticleHandle(InParticle1)->OneWayInteraction();
			if (bIsOneWay0 && bIsOneWay1)
			{
				return EParticlePairMidPhaseType::SphereApproximation;
			}
		}

		// If we have two small flat hierarchies we will expand and prefilter all shape pairs
		if (bCanPrebuildShapePairs)
		{
			return EParticlePairMidPhaseType::ShapePair;
		}

		// We have at least one complicated geometry hierarchy so use the general purpose midphase
		return EParticlePairMidPhaseType::Generic;
	}

	FParticlePairMidPhase* FParticlePairMidPhase::Make(FGeometryParticleHandle* InParticle0, FGeometryParticleHandle* InParticle1)
	{
		const EParticlePairMidPhaseType MidPhaseType = CalculateMidPhaseType(InParticle0, InParticle1);
		switch (MidPhaseType)
		{
			case EParticlePairMidPhaseType::Generic:
				return new FGenericParticlePairMidPhase();
			case EParticlePairMidPhaseType::ShapePair:
				return new FShapePairParticlePairMidPhase();
			case EParticlePairMidPhaseType::SphereApproximation:
				return new FSphereApproximationParticlePairMidPhase();
			default:
				check(false);
				return nullptr;
		}
	}

	void FParticlePairMidPhase::DetachParticle(FGeometryParticleHandle* Particle)
	{
		Reset();

		if (Particle == Particle0)
		{
			Particle0 = nullptr;
		}
		else if (Particle == Particle1)
		{
			Particle1 = nullptr;
		}
	}

	void FParticlePairMidPhase::Reset()
	{
		Flags.bIsActive = true;
		Flags.bIsCCD = false;
		Flags.bIsCCDActive = false;
		Flags.bIsMACD = false;
		Flags.bIsSleeping = false;
		Flags.bIsModified = false;
		Flags.bIsConvexOptimizationActive = true;
		Flags.bIsConvexOptimizationUsed = false;

		ResetImpl();
	}

	void FParticlePairMidPhase::ResetModifications()
	{
		if (Flags.bIsModified)
		{
			Flags.bIsActive = true;
			Flags.bIsCCDActive = Flags.bIsCCD;
			Flags.bIsConvexOptimizationActive = true;
			Flags.bIsModified = false;
		}
	}

	void FParticlePairMidPhase::Init(
		FGeometryParticleHandle* InParticle0,
		FGeometryParticleHandle* InParticle1,
		const Private::FCollisionParticlePairKey& InParticlePairKey,
		const FCollisionContext& Context)
	{
		PHYSICS_CSV_SCOPED_EXPENSIVE(PhysicsVerbose, NarrowPhase_Filter);
		check(InParticle0 != nullptr);
		check(InParticle1 != nullptr);

		Particle0 = InParticle0;
		Particle1 = InParticle1;
		ParticlePairKey = InParticlePairKey;

		Flags.bIsActive = true;

		FConstGenericParticleHandle P0 = Particle0;
		FConstGenericParticleHandle P1 = Particle1;

		// If CCD is allowed in the current context and for at least one of
		// the particles involved, enable it for this midphase.
		//
		// bIsCCDActive is reset to bIsCCD each frame, but can be overridden
		// by modifiers.
		const bool bIsCCD = Context.GetSettings().bAllowCCD && (P0->CCDEnabled() || P1->CCDEnabled());
		Flags.bIsCCD = bIsCCD;
		Flags.bIsCCDActive = bIsCCD;

		// Enable Motion-Aware Collision Detection if either particle requests it
		// (NOTE: MACD also affects how the particle world-space bounds is expanded in the broadphase)
		const bool bIsMACD = Context.GetSettings().bAllowMACD && (CVars::bChaosForceMACD || P0->MACDEnabled() || P1->MACDEnabled());
		Flags.bIsMACD = bIsMACD;

		// Initially we allow for convex optimization where available
		Flags.bIsConvexOptimizationActive = true;

		BuildDetectorsImpl();

		InitThresholds();
	}

	bool FParticlePairMidPhase::ShouldEnableCCDSweep(const FReal Dt)
	{
		// bIsCCDActive is set to bIsCCD at the beginning of every frame, but may be
		// overridden in midphase modification or potentially other systems which run
		// in between mid and narrow phase. bIsCCDActive indicates the final
		// overridden value so we use that here instead of bIsCCD.
		if (Flags.bIsCCDActive != 0)
		{
			FConstGenericParticleHandle ConstParticle0 = FConstGenericParticleHandle(Particle0);
			FConstGenericParticleHandle ConstParticle1 = FConstGenericParticleHandle(Particle1);

			// See comments in FSingleShapePairCollisionDetector::GenerateCollisionCCDImpl for why we use V() here
			FReal LengthCCD = 0;
			FVec3 DirCCD = FVec3(0);
			const FVec3 DeltaX0 = ConstParticle0->V() * Dt;
			const FVec3 DeltaX1 = ConstParticle1->V() * Dt;
			const bool bUseCCD = Collisions::ShouldUseCCD(Particle0, DeltaX0, Particle1, DeltaX1, DirCCD, LengthCCD);

			return bUseCCD;
		}
		return false;
	}

	void FParticlePairMidPhase::InitThresholds()
	{
		// @todo(chaos): improve this threshold calculation for thin objects? Dynamic thin objects have bigger problems so maybe we don't care
		// @todo(chaos): Spheres and capsules need smaller position tolerance - the restore test doesn't work well with rolling
		const bool bIsDynamic0 = FConstGenericParticleHandle(Particle0)->IsDynamic();
		const bool bIsDynamic1 = FConstGenericParticleHandle(Particle1)->IsDynamic();
		const bool bIsBounded0 = Particle0->HasBounds();
		const bool bIsBounded1 = Particle1->HasBounds();

		// NOTE: If CullDistance ends up smaller than the thresholds used to restore collisions, we can end up missing
		// collisions as the objects move if we restore a "zero contact" manifold after movement greater than the cull distance. 
		// Currently this should not happen, but it is not explicitly ensured by the way the thresholds and CullDistanceScale are calculated.
		// @todo(chaos): Add a way to enforce a CullDistance big enough to support the reuse thresholds
		const FReal CullDistanceReferenceSizeInv = FReal(Chaos_Collision_CullDistanceScaleInverseSize);
		const FReal MinCullDistanceScale = FReal(Chaos_Collision_MinCullDistanceScale);
		const FReal MaxBoundsSize0 = (bIsDynamic0 && bIsBounded0) ? Particle0->LocalBounds().Extents().GetMax() : FReal(0);
		const FReal MaxBoundsSize1 = (bIsDynamic1 && bIsBounded1) ? Particle1->LocalBounds().Extents().GetMax() : FReal(0);
		const FReal CullDistanceScale0 = MaxBoundsSize0 * CullDistanceReferenceSizeInv;
		const FReal CullDistanceScale1 = MaxBoundsSize1 * CullDistanceReferenceSizeInv;
		CullDistanceScale = (FRealSingle)FMath::Max3(CullDistanceScale0, CullDistanceScale1, MinCullDistanceScale);
	}

	void FParticlePairMidPhase::GenerateCollisions(
		const FReal InCullDistance,
		const FReal InDt,
		const FCollisionContext& Context)
	{
		if (!IsValid())
		{
			return;
		}

		CHAOS_MIDPHASE_SCOPE_CYCLE_TIMER(FParticlePairMidPhase_GenerateCollision);

		if (Flags.bIsActive)
		{
			FConstGenericParticleHandle P0 = GetParticle0();
			FConstGenericParticleHandle P1 = GetParticle1();

			FRealSingle Dt = FRealSingle(InDt);

			// CullDistance is scaled by the size of the dynamic objects.
			FRealSingle CullDistance = FRealSingle(InCullDistance) * CullDistanceScale;

			// If CCD is enabled, did we move far enough to require a sweep?
			Flags.bUseSweep = (Flags.bIsCCDActive != 0) && ShouldEnableCCDSweep(Dt);

#if !UE_BUILD_TEST && !UE_BUILD_SHIPPING
			// At least one body must be dynamic, and at least one must be awake (or moving if a kinematic)
			const bool bIsMoving0 = (P0->IsDynamic() && !P0->IsSleeping()) || P0->IsMovingKinematic();
			const bool bIsMoving1 = (P1->IsDynamic() && !P1->IsSleeping()) || P1->IsMovingKinematic();
			const bool bAnyMoving = bIsMoving0 || bIsMoving1;
			ensureMsgf(bAnyMoving, TEXT("GenerateCollisions called on two stationary objects %s %s"), *P0->GetDebugName(), *P1->GetDebugName());
#endif

			FVec3f RelativeMovement = FVec3f(0);
			if (!Flags.bIsMACD)
			{
				// We increase CullDistance based on velocity (up to a limit for perf with large velocities).
				// NOTE: This somewhat matches the bounds expansion in FPBDRigidsEvolutionGBF::Integrate
				// NOTE: We use PreV here which is the velocity after collisions from the previous tick because we want the 
				// velocity without gravity from this tick applied. This is mainly so that we get the same CullDistance from 
				// one tick to the next, even if one of the particles goes to sleep, and therefore its velocity is now zero 
				// because gravity is no longer applied. Also see FPBDIslandManager::PropagateIslandSleep for other issues 
				// related to velocity and sleeping...
				// NOTE: we used to extend the cull distance for CCD objects, but this is no longer required. The sweep and
				// rewind phase does not use CullDistance, and once rewound we are using normal collision detection where
				// an expanded CullDistance doesn't help and makes perf worse.
				const FRealSingle VMax0 = P0->GetPreVf().GetAbsMax();
				const FRealSingle VMax1 = P1->GetPreVf().GetAbsMax();
				const FRealSingle VMaxDt = FMath::Max(VMax0, VMax1) * Dt;
				const FRealSingle VelocityBoundsMultiplier = FRealSingle(Context.GetSettings().BoundsVelocityInflation);
				const FRealSingle MaxVelocityBoundsExpansion = FRealSingle(Context.GetSettings().MaxVelocityBoundsExpansion);
				if ((VelocityBoundsMultiplier > 0) && (MaxVelocityBoundsExpansion > 0))
				{
					CullDistance += FMath::Min(VelocityBoundsMultiplier * VMaxDt, MaxVelocityBoundsExpansion);
				}
			}
			else
			{
				// Movement-aware collision detection (MACD) takes the relative position change this tick as input
				// We do not expand CullDistance. Depending on the shape pair types involved, the collision detection
				// step will either pad the CullDistance itself, or ideally compare RelativeMovement with the contact 
				// normal when determining what contacts to cull. Eventually all shape pairs should do the latter.
				RelativeMovement = (P0->GetVf() - P1->GetVf()) * Dt;
			}

			// Run collision detection on all potentially colliding shape pairs
			NumActiveConstraints = GenerateCollisionsImpl(Dt, CullDistance, RelativeMovement, Context);
		}

		// Reset any modifications applied by the MidPhaseModifier.
		// In principle this belongs at the start of the frame but it is hard to avoid the cache miss there.
		ResetModifications();

		LastUsedEpoch = Context.GetAllocator()->GetCurrentEpoch();
	}

	void FParticlePairMidPhase::InjectCollision(const FPBDCollisionConstraint& Constraint, const FCollisionContext& Context)
	{
		InjectCollisionImpl(Constraint, Context);

		LastUsedEpoch = Context.GetAllocator()->GetCurrentEpoch();
	}

	void FParticlePairMidPhase::SetIsSleeping(const bool bInIsSleeping, const int32 CurrentEpoch)
	{
		// This can be called from two locations:
		// 1)	At the start of the tick as a results of some state change from the game thread such as an explicit wake event,
		//		applying an impulse, or moving a particle.
		// 2)	After the constraint solver phase when we put non-moving islands to sleep.
		// 
		// Note that in both cases there is a collision detection phase before the next constraint solving phase.
		//
		// When awakening we re-activate collisions so that we have a "previous" collision to use for static friction etc.
		// We don't need to do anything when going to sleep because sleeping particles pairs are ignored in collision detection 
		// so the next set of active collisions generated will not contain these collisions.

		if (Flags.bIsSleeping != bInIsSleeping)
		{
			// If we are waking particles, reactivate all collisions that were
			// active when we were put to sleep, i.e., all collisions whose LastUsedEpoch
			// is equal to our LastUsedEpoch.
			const bool bWakingUp = !bInIsSleeping;
			if (bWakingUp)
			{
				if (LastUsedEpoch < CurrentEpoch)
				{
					// Restore all constraints that were active when we were put to sleep
					WakeCollisionsImpl(CurrentEpoch);

					LastUsedEpoch = CurrentEpoch;
				}
			}
			// If we are going to sleep, there is nothing to do (see comments above)

			Flags.bIsSleeping = bInIsSleeping;
		}
	}

	bool FParticlePairMidPhase::IsInConstraintGraph() const
	{
		// @todo(chaos): optimize
		bool bInGraph = false;
		VisitConstCollisions([&bInGraph](const FPBDCollisionConstraint& Constraint)
		{
			if (Constraint.GetConstraintGraphEdge() != nullptr)
			{
				bInGraph = true;
				return ECollisionVisitorResult::Stop;
			}
			return ECollisionVisitorResult::Continue;
		}, ECollisionVisitorFlags::VisitAllCurrent);
		return bInGraph;
	}


	////////////////////////////////////////////////////////////////////////////////////////
	////////////////////////////////////////////////////////////////////////////////////////
	////////////////////////////////////////////////////////////////////////////////////////
	////////////////////////////////////////////////////////////////////////////////////////


	FShapePairParticlePairMidPhase::FShapePairParticlePairMidPhase()
		: FParticlePairMidPhase(EParticlePairMidPhaseType::ShapePair)
	{

	}

	void FShapePairParticlePairMidPhase::ResetImpl()
	{
		ShapePairDetectors.Reset();
	}

	void FShapePairParticlePairMidPhase::BuildDetectorsImpl()
	{
		if (IsValid() && (Particle0 != Particle1))
		{
			const FShapesArray& Shapes0 = Particle0->ShapesArray();
			const FShapesArray& Shapes1 = Particle1->ShapesArray();
			for (int32 ShapeIndex0 = 0; ShapeIndex0 < Shapes0.Num(); ++ShapeIndex0)
			{
				const FPerShapeData* Shape0 = Shapes0[ShapeIndex0].Get();
				for (int32 ShapeIndex1 = 0; ShapeIndex1 < Shapes1.Num(); ++ShapeIndex1)
				{
					const FPerShapeData* Shape1 = Shapes1[ShapeIndex1].Get();
					TryAddShapePair(Shape0, ShapeIndex0, Shape1, ShapeIndex1);
				}
			}
		}
	}

	void FShapePairParticlePairMidPhase::TryAddShapePair(const FPerShapeData* Shape0, const int32 ShapeIndex0, const FPerShapeData* Shape1, const int32 ShapeIndex1)
	{
		const FImplicitObject* Implicit0 = Shape0->GetLeafGeometry();
		const EImplicitObjectType ImplicitType0 = Private::GetImplicitCollisionType(Particle0, Implicit0);

		const FImplicitObject* Implicit1 = Shape1->GetLeafGeometry();
		const EImplicitObjectType ImplicitType1 = Private::GetImplicitCollisionType(Particle1, Implicit1);

		const bool bDoPassFilter = ShapePairNarrowPhaseFilter(ImplicitType0, Shape0, ImplicitType1, Shape1);
		if (bDoPassFilter)
		{
			bool bSwap = false;
			const EContactShapesType ShapePairType = Collisions::CalculateShapePairType(Particle0, Implicit0, Particle1, Implicit1, bSwap);

			if (ShapePairType != EContactShapesType::Unknown)
			{
				if (!bSwap)
				{
					const Private::FCollisionSortKey CollisionSortKey = Private::FCollisionSortKey(Particle0, ShapeIndex0, Particle1, ShapeIndex1);
					ShapePairDetectors.Emplace(FSingleShapePairCollisionDetector(Particle0, Shape0, Particle1, Shape1, CollisionSortKey, ShapePairType, *this));
				}
				else
				{
					const Private::FCollisionSortKey CollisionSortKey = Private::FCollisionSortKey(Particle1, ShapeIndex1, Particle0, ShapeIndex0);
					ShapePairDetectors.Emplace(FSingleShapePairCollisionDetector(Particle1, Shape1, Particle0, Shape0, CollisionSortKey, ShapePairType, *this));
				}
			}
			else
			{
				// If we get here we have a Union of Unions, but that means we should have created 
				// a FGenericParticlePairMidPhase. See CalculateMidPhaseType
				ensure(false);
			}
		}
	}

	int32 FShapePairParticlePairMidPhase::GenerateCollisionsImpl(
		const FRealSingle Dt,
		const FRealSingle CullDistance,
		const FVec3f& RelativeMovement,
		const FCollisionContext& Context)
	{
		CHAOS_MIDPHASE_SCOPE_CYCLE_TIMER(FShapePairParticlePairMidPhase_GenerateCollision);

		//TRACE_COUNTER_INCREMENT(ChaosTraceCounter_MidPhase_NumShapePair);

		int32 NumActive = 0;
		if (Flags.bIsCCDActive != 0)
		{
			for (FSingleShapePairCollisionDetector& ShapePair : ShapePairDetectors)
			{
				NumActive += ShapePair.GenerateCollisionCCD(Dt, CullDistance, RelativeMovement, Flags.bUseSweep, Context);
			}
		}
		else
		{
			for (FSingleShapePairCollisionDetector& ShapePair : ShapePairDetectors)
			{
				NumActive += ShapePair.GenerateCollision(Dt, CullDistance, RelativeMovement, Context);
			}
		}
		return NumActive;
	}

	void FShapePairParticlePairMidPhase::WakeCollisionsImpl(const int32 CurrentEpoch)
	{
		for (FSingleShapePairCollisionDetector& ShapePair : ShapePairDetectors)
		{
			ShapePair.WakeCollision(LastUsedEpoch, CurrentEpoch);
		}
	}

	void FShapePairParticlePairMidPhase::InjectCollisionImpl(const FPBDCollisionConstraint& Constraint, const FCollisionContext& Context)
	{
		const FPerShapeData* Shape0 = Constraint.GetShape0();
		const FPerShapeData* Shape1 = Constraint.GetShape1();

		// @todo(chaos): fix O(N) search for shape pair - store the index in the cookie (it will be the same
		// as long as the ShapesArray on each particle has not changed)
		for (FSingleShapePairCollisionDetector& ShapePair : ShapePairDetectors)
		{
			if (((Shape0 == ShapePair.GetShape0()) && (Shape1 == ShapePair.GetShape1())) || ((Shape0 == ShapePair.GetShape1()) && (Shape1 == ShapePair.GetShape0())))
			{
				ShapePair.SetCollision(Constraint, Context);
				break;
			}
		}
	}

	////////////////////////////////////////////////////////////////////////////////////////
	////////////////////////////////////////////////////////////////////////////////////////
	////////////////////////////////////////////////////////////////////////////////////////
	////////////////////////////////////////////////////////////////////////////////////////


	FGenericParticlePairMidPhase::FGenericParticlePairMidPhase()
		: FParticlePairMidPhase(EParticlePairMidPhaseType::Generic)
	{
	}

	FGenericParticlePairMidPhase::~FGenericParticlePairMidPhase()
	{
	}

	void FGenericParticlePairMidPhase::ResetImpl()
	{
		Constraints.Empty();
		NewConstraints.Empty();
	}

	void FGenericParticlePairMidPhase::SetConvexOptimizationIsActive(bool bSetConvexOptimizationIsActive, TFunctionRef<void(FPBDCollisionConstraint*)> OnDestroyingCollision)
	{
		Flags.bIsModified = true;
		Flags.bIsConvexOptimizationActive = bSetConvexOptimizationIsActive;

		// We need to reset collisions if we enable/disable use of the convex optimizer because we are switching 
		// to a different set of collision shapes. This is necessary because the "key" used to identify a shape 
		// would not be unique between the two sets of shapes so we would end up with mismatched collisions in the map.
		// 
		// NOTE: Collisions must be removed from the constraint graph before being destroyed. The caller is
		// responsible for this because we don't know about the graph here (and shouldn't because the solver
		// may not be using a graph).
		// @todo(chaos): see if we can clean this up a bit?
		// 
		// NOTE: bIsConvexOptimizationActive can be changed in a modifier which gets reset every frame so we
		// use bIsConvexOptimizationUsed to remember what we actually used rather than just resetting on change.
		if (CVars::bChaosMidPhaseModifierConvexOptimizerFix)
		{
			if (Flags.bIsConvexOptimizationActive != Flags.bIsConvexOptimizationUsed)
			{
				// The caller should remove collisions from the graph here
				for (TMap<uint64, FPBDCollisionConstraintPtr>::ElementType& KVP : Constraints)
				{
					FPBDCollisionConstraintPtr& Constraint = KVP.Value;
					OnDestroyingCollision(Constraint.Get());
				}

				// NOTE: An assert here would indicate that the OnDestroyingCollisions callback did not 
				// remove collisions from the constraint graph.
				Constraints.Reset();
			}
		}
	}

	void FGenericParticlePairMidPhase::BuildDetectorsImpl()
	{
		check(Particle0->GetGeometry() != nullptr);
		check(Particle1->GetGeometry() != nullptr);
	}

	int32 FGenericParticlePairMidPhase::GenerateCollisionsImpl(
		const FRealSingle Dt,
		const FRealSingle CullDistance,
		const FVec3f& RelativeMovement,
		const FCollisionContext& Context)
	{
		CHAOS_MIDPHASE_SCOPE_CYCLE_TIMER(FGenericParticlePairMidPhase_GenerateCollisionImpl);

		//TRACE_COUNTER_INCREMENT(ChaosTraceCounter_MidPhase_NumGeneric);
		const FImplicitObjectRef Implicit0 = GetParticle0()->GetGeometry();
		const FImplicitObjectRef Implicit1 = GetParticle1()->GetGeometry();

		FPBDRigidClusteredParticleHandle* ClusteredHandle0 = GetParticle0()->CastToClustered();
		FPBDRigidClusteredParticleHandle* ClusteredHandle1 = GetParticle1()->CastToClustered();

		Private::FConvexOptimizer* ConvexOptimizer0
			= Flags.bIsConvexOptimizationActive && ClusteredHandle0
			? ClusteredHandle0->ConvexOptimizer().Get()
			: nullptr;
		Private::FConvexOptimizer* ConvexOptimizer1
			= Flags.bIsConvexOptimizationActive && ClusteredHandle1
			? ClusteredHandle1->ConvexOptimizer().Get()
			: nullptr;

		// bIsConvexOptimizationActive can be changed in a modifier which gets reset every frame so we
		// need to remember what we actually used when generating collisions, so that we know if we
		// have to destroy those collisions if it changes again (see SetConvexOptimizationIsActive)
		// NOTE: We don't have to handle the case where bHasConvexOptimizer0 or bHasConvexOptimizer1
		// changes because we already reset collisions whenever geometry is replaced or rebuilt.
		const bool bHasConvexOptimizer0 = (ConvexOptimizer0 != nullptr) && ConvexOptimizer0->IsValid();
		const bool bHasConvexOptimizer1 = (ConvexOptimizer1 != nullptr) && ConvexOptimizer1->IsValid();
		Flags.bIsConvexOptimizationUsed = (bHasConvexOptimizer0 || bHasConvexOptimizer1);

		// See if we have a BVH for either/both of the particles
		const Private::FImplicitBVH* BVH0 = nullptr;
		const Private::FImplicitBVH* BVH1 = nullptr;
		if (CVars::bChaosUnionBVHEnabled)
		{
			if (const FImplicitObjectUnion* Union0 = Implicit0->template AsA<FImplicitObjectUnion>())
			{
				if(!bHasConvexOptimizer0)
				{
					BVH0 = Union0->GetBVH();
				}
			}
			if (const FImplicitObjectUnion* Union1 = Implicit1->template AsA<FImplicitObjectUnion>())
			{
				if(!bHasConvexOptimizer1)
				{
					BVH1 = Union1->GetBVH();
				}
			}
		}

		// Create constraints for all the shape pairs whose bounds overlap.
		// If we have a BVH, use it. Otherwise run a recursive hierarchy sweep.
		// @todo(chaos): if we have 2 BVHs select the deepest one as BVHA?
		if ((BVH0 != nullptr) && (BVH1 != nullptr))
		{
			GenerateCollisionsBVHBVH(GetParticle0(), BVH0, GetParticle1(), BVH1, CullDistance, Dt, Context);
		}
		else if (BVH0 != nullptr)
		{
			GenerateCollisionsBVHImplicitHierarchy(GetParticle0(), BVH0, GetParticle1(), Implicit1, ConvexOptimizer1, CullDistance, Dt, Context);
		}
		else if (BVH1 != nullptr)
		{
			GenerateCollisionsBVHImplicitHierarchy(GetParticle1(), BVH1, GetParticle0(), Implicit0, ConvexOptimizer0, CullDistance, Dt, Context);
		}
		else
		{
			GenerateCollisionsImplicitHierarchyImplicitHierarchy(GetParticle0(), Implicit0, ConvexOptimizer0, GetParticle1(), Implicit1, ConvexOptimizer1, CullDistance, Dt, Context);
		}

		// Generate manifolds for each constraint we created/recovered and (re)activate if necessary
		int32 NumActive = 0;
		if (NewConstraints.Num() > 0)
		{
			NumActive = ProcessNewConstraints(CullDistance, Dt, Context);
		}

		// @todo(chaos): we could clean up unused collisions between this pair, but probably not worth it
		//PruneConstraints();

		return NumActive;
	}

	// Detect collisions between two BVHs
	void FGenericParticlePairMidPhase::GenerateCollisionsBVHBVH(
		FGeometryParticleHandle* ParticleA, const Private::FImplicitBVH* BVHA,
		FGeometryParticleHandle* ParticleB, const Private::FImplicitBVH* BVHB,
		const FReal CullDistance,
		const FReal Dt,
		const FCollisionContext& Context)
	{
		CHAOS_MIDPHASE_SCOPE_CYCLE_TIMER(FGenericParticlePairMidPhase_GenerateCollision_BVHBVH);

		const FRigidTransform3 ParticleWorldTransformA = FConstGenericParticleHandle(ParticleA)->GetTransformPQ();
		const FRigidTransform3 ParticleWorldTransformB = FConstGenericParticleHandle(ParticleB)->GetTransformPQ();
		const FRigidTransform3 ParticleTransformBToA = ParticleWorldTransformB.GetRelativeTransform(ParticleWorldTransformA);

		const FShapeInstanceArray& ShapeInstancesA = ParticleA->ShapeInstances();
		const FShapeInstanceArray& ShapeInstancesB = ParticleB->ShapeInstances();

		// Visit all overlapping leaf node pairs in BVHA and BVHB
		Private::FImplicitBVH::VisitOverlappingLeafNodes(*BVHA, *BVHB, ParticleTransformBToA, 
			[this, ParticleA, BVHA, &ParticleWorldTransformA, &ShapeInstancesA, ParticleB,
			BVHB, &ParticleWorldTransformB, &ShapeInstancesB,
			CullDistance, Dt, &Context]
			(const int32 NodeIndexA, const int32 NodeIndexB)
			{
				// If we get here, BVHA(NodeIndexA) and BVHB(NodeIndexB) overlap so we must collide all object pairs in the two nodes
				BVHA->VisitNodeObjects(NodeIndexA,
					[this, ParticleA, &ParticleWorldTransformA, &ShapeInstancesA, ParticleB,
					BVHB, &ParticleWorldTransformB, &ShapeInstancesB, NodeIndexB,
					CullDistance, Dt, &Context]
					(const FImplicitObject* ImplicitA, const FRigidTransform3f& RelativeTransformfA, const FAABB3f& RelativeBoundsfA, const int32 RootObjectIndexA, const int32 LeafObjectIndexA) -> void
					{
						const FShapeInstance* ShapeInstanceA = GetShapeInstance(ShapeInstancesA, RootObjectIndexA);
						if (!FilterHasSimEnabled(ShapeInstanceA)) return;
						
						const FRigidTransform3 RelativeTransformA = FRigidTransform3(RelativeTransformfA);
						BVHB->VisitNodeObjects(NodeIndexB,
							[this, ParticleA, ImplicitA, ShapeInstanceA, &ParticleWorldTransformA, &RelativeTransformA, LeafObjectIndexA,
							ParticleB, BVHB, &ParticleWorldTransformB, &ShapeInstancesB,
							CullDistance, Dt, &Context]
							(const FImplicitObject* ImplicitB, const FRigidTransform3f& RelativeTransformfB, const FAABB3f& RelativeBoundsfB, const int32 RootObjectIndexB, const int32 LeafObjectIndexB) -> void
							{
								const FShapeInstance* ShapeInstanceB = GetShapeInstance(ShapeInstancesB, RootObjectIndexB);
								if (!FilterHasSimEnabled(ShapeInstanceB)) return;
								
								const FRigidTransform3 RelativeTransformB = FRigidTransform3(RelativeTransformfB);
								// Detect collisions between the single implicit object pair
								GenerateCollisionsImplicitLeafImplicitLeaf(
									ParticleA, ImplicitA, ShapeInstanceA, ParticleWorldTransformA, RelativeTransformA, LeafObjectIndexA,
									ParticleB, ImplicitB, ShapeInstanceB, ParticleWorldTransformB, RelativeTransformB, LeafObjectIndexB,
									CullDistance, Dt, Context);
							});
					});
			});
	}

	// Detect collisions between a BVH and some other implicit object (which may be a hierarchy)
	void FGenericParticlePairMidPhase::GenerateCollisionsBVHImplicitHierarchy(
		FGeometryParticleHandle* ParticleA, const Private::FImplicitBVH* BVHA,
		FGeometryParticleHandle* ParticleB, const FImplicitObject* RootImplicitB, const Private::FConvexOptimizer* ConvexOptimizerB,
		const FReal CullDistance, const FReal Dt, const FCollisionContext& Context)
	{
		CHAOS_MIDPHASE_SCOPE_CYCLE_TIMER(FGenericParticlePairMidPhase_GenerateCollision_BVHImplicitHierarchy);

		const FShapeInstanceArray& ShapeInstancesB = ParticleB->ShapeInstances();

		// Visit all the leaf implicits in RootImplicitB and collide against the BVH
		VisitCollisionObjects(ConvexOptimizerB, RootImplicitB,
			[this, ParticleA, BVHA, ParticleB, &ShapeInstancesB, CullDistance, Dt, &Context, &ConvexOptimizerB]
			(const FImplicitObject* ImplicitB, const FRigidTransform3& RelativeTransformB, const int32 RootObjectIndexB, const int32 ObjectIndexB, const int32 LeafObjectIndexB) -> void
			{
				const FShapeInstance* ShapeInstanceB = GetShapeInstance(ShapeInstancesB, RootObjectIndexB, ConvexOptimizerB);
				if (!FilterHasSimEnabled(ShapeInstanceB)) return;

				// ImplicitB is a single object. We perform the bounds tests in A space
				GenerateCollisionsBVHImplicitLeaf(
					ParticleA, BVHA,
					ParticleB, ImplicitB, ShapeInstanceB, RelativeTransformB, LeafObjectIndexB,
					CullDistance, Dt, Context);
			});
	}

	// Detect collisions between two implicits, where either or both may be a hierarchy, but neither has a BVH
	void FGenericParticlePairMidPhase::GenerateCollisionsImplicitHierarchyImplicitHierarchy(
		FGeometryParticleHandle* ParticleA, const FImplicitObject* RootImplicitA, const Private::FConvexOptimizer* ConvexOptimizerA,
		FGeometryParticleHandle* ParticleB, const FImplicitObject* RootImplicitB, const Private::FConvexOptimizer* ConvexOptimizerB,
		const FReal CullDistance,
		const FReal Dt,
		const FCollisionContext& Context)
	{
		CHAOS_MIDPHASE_SCOPE_CYCLE_TIMER(FGenericParticlePairMidPhase_GenerateCollision_ImplicitHierarchyImplicitHierarch);

		const FConstGenericParticleHandle PA = ParticleA;
		const FConstGenericParticleHandle PB = ParticleB;
		const FShapeInstanceArray& ShapeInstancesA = ParticleA->ShapeInstances();
		const FShapeInstanceArray& ShapeInstancesB = ParticleB->ShapeInstances();

		// Particle transforms
		const FRigidTransform3 ParticleWorldTransformA = PA->GetTransformPQ();
		const FRigidTransform3 ParticleWorldTransformB = PB->GetTransformPQ();
		const FRigidTransform3 ParticleTransformAToB = ParticleWorldTransformA.GetRelativeTransform(ParticleWorldTransformB);

		// Detect collisions between Implicit Hierarchy of ParticleA and Implicit Hierarchy of ParticleB
		// Given an ImplicitObject from ParticleA (which we know overlaps the bounds of some parts of ParticleB),
		// run collision detection on ImplicitA against the implicit object hierarchy of ParticleB.
		VisitCollisionObjects(ConvexOptimizerA, RootImplicitA, [this, ParticleA, &ShapeInstancesA, &ParticleWorldTransformA,
			ParticleB, &ShapeInstancesB, RootImplicitB, &ParticleWorldTransformB, &ParticleTransformAToB,
			CullDistance, Dt, &Context, &ConvexOptimizerA, &ConvexOptimizerB]
			(const FImplicitObject* ImplicitA, const FRigidTransform3& RelativeTransformA, const int32 RootObjectIndexA, const int32 ObjectIndexA, const int32 LeafObjectIndexA)
			{
				const FShapeInstance* ShapeInstanceA = GetShapeInstance(ShapeInstancesA, RootObjectIndexA, ConvexOptimizerA);
				if (!FilterHasSimEnabled(ShapeInstanceA)) return;

				const FAABB3 RelativeBoundsA = ImplicitA->CalculateTransformedBounds(RelativeTransformA);
				const FAABB3 ShapeBoundsAInB = RelativeBoundsA.TransformedAABB(ParticleTransformAToB).Thicken(CullDistance);

				// Detect collisions between ImplicitA and Implicit Hierarchy of ParticleB
				VisitOverlappingObjects(ConvexOptimizerB, RootImplicitB, ShapeBoundsAInB,
					[this, ParticleA, ImplicitA, ShapeInstanceA, &ParticleWorldTransformA, &RelativeTransformA, LeafObjectIndexA,
					ParticleB, &ParticleWorldTransformB, &ShapeInstancesB,
					CullDistance, Dt, &Context, &ConvexOptimizerB]
					(const FImplicitObject* ImplicitB, const FRigidTransform3& RelativeTransformB, const int32 RootObjectIndexB, const int32 ObjectIndexB, const int32 LeafObjectIndexB)
					{
						const FShapeInstance* ShapeInstanceB = GetShapeInstance(ShapeInstancesB, RootObjectIndexB, ConvexOptimizerB);
						if (!FilterHasSimEnabled(ShapeInstanceB)) return;

						// Detect collisions between ImplicitA and ImplicitB (both leaf implicits)
						GenerateCollisionsImplicitLeafImplicitLeaf(
							ParticleA, ImplicitA, ShapeInstanceA, ParticleWorldTransformA, RelativeTransformA, LeafObjectIndexA,
							ParticleB, ImplicitB, ShapeInstanceB, ParticleWorldTransformB, RelativeTransformB, LeafObjectIndexB,
							CullDistance, Dt, Context);
					});
			});
	}

	// Detect collisions between a BVH and a leaf implicit object (not a hierarchy)
	void FGenericParticlePairMidPhase::GenerateCollisionsBVHImplicitLeaf(
		FGeometryParticleHandle* ParticleA, const Private::FImplicitBVH* BVHA,
		FGeometryParticleHandle* ParticleB, const FImplicitObject* ImplicitB, const FShapeInstance* ShapeInstanceB, const FRigidTransform3& RelativeTransformB, const int32 LeafObjectIndexB,
		const FReal CullDistance, const FReal Dt, const FCollisionContext& Context)
	{
		CHAOS_MIDPHASE_SCOPE_CYCLE_TIMER(FGenericParticlePairMidPhase_GenerateCollision_BVHImplicitLeaf);

		const FConstGenericParticleHandle PA = ParticleA;
		const FConstGenericParticleHandle PB = ParticleB;

		// Particle transforms
		const FRigidTransform3 ParticleWorldTransformA = PA->GetTransformPQ();
		const FRigidTransform3 ParticleWorldTransformB = PB->GetTransformPQ();
		const FShapeInstanceArray& ShapeInstancesA = ParticleA->ShapeInstances();

		// ImplicitB transforms/bounds (expanded by cull distance)
		const FRigidTransform3 ImplicitTransformB = RelativeTransformB * ParticleWorldTransformB;
		const FRigidTransform3 ImplicitTransformBToA = ImplicitTransformB.GetRelativeTransform(ParticleWorldTransformA);
		const FAABB3 ImplicitBoundsBInA = ImplicitB->CalculateTransformedBounds(ImplicitTransformBToA).Thicken(CullDistance);

		// If ImplicitB has a built-in BVH (Heightfield or TriMesh) we handle the test against the BVH differently
		const bool bHasInternalBVHB = ImplicitB->template IsA<FHeightField>();

		// Visitor for FImplicitBVH::VisitNodeObjects
		// Given an ImplicitObject from ParticleA (which we know overlaps the bounds of ImplicitB),
		// run collision detection on ImplicitA against ImplicitB.
		const auto& NodeObjectVisitorA =
			[this, ParticleA, &ShapeInstancesA, &ParticleWorldTransformA, 
			ParticleB, ImplicitB, ShapeInstanceB, &ParticleWorldTransformB, &RelativeTransformB, LeafObjectIndexB, 
			CullDistance, Dt, &Context]
			(const FImplicitObject* ImplicitA, const FRigidTransform3f& RelativeTransformfA, const FAABB3f& RelativeBoundsfA, const int32 RootObjectIndexA, const int32 LeafObjectIndexA) -> void
			{
				const FShapeInstance* ShapeInstanceA = GetShapeInstance(ShapeInstancesA, RootObjectIndexA);
				if (!FilterHasSimEnabled(ShapeInstanceA)) return;
				
				const FRigidTransform3 RelativeTransformA = FRigidTransform3(RelativeTransformfA);

				GenerateCollisionsImplicitLeafImplicitLeaf(
					ParticleA, ImplicitA, ShapeInstanceA, ParticleWorldTransformA, RelativeTransformA, LeafObjectIndexA,
					ParticleB, ImplicitB, ShapeInstanceB, ParticleWorldTransformB, RelativeTransformB, LeafObjectIndexB,
					CullDistance, Dt, Context);
			};

		// Visitor for FImplicitBVH::VisitNodes
		// This will be passed the nodes of the BVH on ParticleA and its bounds and contents. If the bounds overlap something in ParticleB
		// we will keep recursing into the BVH. When we hit a leaf, run collision detection between the leaf contents and ParticleB.
		const auto& NodeVisitorA =
			[BVHA, ImplicitB, &ImplicitBoundsBInA, &ImplicitTransformBToA, bHasInternalBVHB, &NodeObjectVisitorA]
			(const FAABB3f& RelativeNodeBoundsfA, const int32 NodeDepthA, const int32 NodeIndexA) -> bool
			{
				const FAABB3 RelativeNodeBoundsA = FAABB3(RelativeNodeBoundsfA);

				if (bHasInternalBVHB)
				{
					// ImplicitB has an internal BVH (e.g., HeightField). We perform the bounds tests in B space
					// NOTE: IsOverlappingBounds performs a deep bounds test, using any internal BVH present on ImplicitB
					const FAABB3 NodeBoundsAInB = RelativeNodeBoundsA.InverseTransformedAABB(ImplicitTransformBToA);
					if (!ImplicitB->IsOverlappingBounds(NodeBoundsAInB))
					{
						return false;
					}
				}
				else
				{
					// ImplicitB is a single object. We perform the bounds tests in A space to avoid the node bounds transform
					if (!ImplicitBoundsBInA.Intersects(RelativeNodeBoundsA))
					{
						return false;
					}
				}

				// Is this node is a leaf, or is entirely contained in B visit all objects in the node and collide against ImplicitB
				// @todo(chaos): support full containment check (calculated in branch above)
				const bool bIsLeafA = BVHA->NodeIsLeaf(NodeIndexA);
				if (bIsLeafA)
				{
					BVHA->VisitNodeObjects(NodeIndexA, NodeObjectVisitorA);
				}

				// Keep recursing
				return true;
			};

		// Visit all the nodes in BVHA and detect collisions with ParticleB
		BVHA->VisitNodes(NodeVisitorA);  
	}

	// Generate collisions between two leaf (not hierarchy) implicits
	void FGenericParticlePairMidPhase::GenerateCollisionsImplicitLeafImplicitLeaf(
		FGeometryParticleHandle* ParticleA, const FImplicitObject* ImplicitA, const FShapeInstance* ShapeInstanceA, const FRigidTransform3 ParticleWorldTransformA, const FRigidTransform3& RelativeTransformA, const int32 LeafObjectIndexA,
		FGeometryParticleHandle* ParticleB, const FImplicitObject* ImplicitB, const FShapeInstance* ShapeInstanceB, const FRigidTransform3 ParticleWorldTransformB, const FRigidTransform3& RelativeTransformB, const int32 LeafObjectIndexB,
		const FReal CullDistance,
		const FReal Dt,
		const FCollisionContext& Context)
	{
		CHAOS_MIDPHASE_SCOPE_CYCLE_TIMER(FGenericParticlePairMidPhase_GenerateCollision_ImplicitLeafImplicitLeaf);

		// Check the sim filter to see if these shapes collide
		const EImplicitObjectType ImplicitTypeA = Private::GetImplicitCollisionType(ParticleA, ImplicitA);
		const EImplicitObjectType ImplicitTypeB = Private::GetImplicitCollisionType(ParticleB, ImplicitB);
		const bool bDoPassFilter = ShapePairNarrowPhaseFilter(ImplicitTypeA, ShapeInstanceA, ImplicitTypeB, ShapeInstanceB);
		if (!bDoPassFilter)
		{
			return;
		}

		// Calculate which bounds tests we should run for this shape pair
		// @todo(chaos): it is too expensive to call this all the time. We should probably build it into DoBoundsOverlap and only perform operation when we pass te AABB test
		//Private::FImplicitBoundsTestFlags BoundsTestFlags = Private::CalculateImplicitBoundsTestFlags(ParticleA, ImplicitA, ShapeInstanceA, ParticleB, ImplicitB, ShapeInstanceB, DistanceCheckSize);
		FRealSingle DistanceCheckSize = 0;
		Private::FImplicitBoundsTestFlags BoundsTestFlags;
		BoundsTestFlags.bEnableAABBCheck = true;
		BoundsTestFlags.bEnableOBBCheck0 = true;
		BoundsTestFlags.bEnableOBBCheck1 = true;

		// Do the objects bounds overlap?
		const bool bDoOverlap = DoBoundsOverlap(
			ImplicitA, ParticleWorldTransformA, RelativeTransformA,
			ImplicitB, ParticleWorldTransformB, RelativeTransformB,
			BoundsTestFlags, DistanceCheckSize, CullDistance);

		if (!bDoOverlap)
		{
			return;
		}

		// Find the BVH Particles if we have LevelSet Collision
		// @todo(chaos): This is not really right. The BVHParticles represent the whole
		// geometry hierarchy from the orginating particle (we may be a ClusterUnion)
		// so we shouldn't be doing a collision per LevelSet
		const FBVHParticles* LevelSetParticlesA = nullptr;
		const FBVHParticles* LevelSetParticlesB = nullptr;
		if (ImplicitA->GetCollisionType() == ImplicitObjectType::LevelSet)
		{
			LevelSetParticlesA = FindLevelSetParticles(ParticleA, ImplicitA);
		}
		if (ImplicitB->GetCollisionType() == ImplicitObjectType::LevelSet)
		{
			LevelSetParticlesB = FindLevelSetParticles(ParticleB, ImplicitB);
		}

		// What shape pair type is this (i.e., which contact update function do we call)
		bool bShouldSwapParticles = false;
		const EContactShapesType ShapePairType = Collisions::CalculateShapePairType(ParticleA, ImplicitA, ParticleB, ImplicitB, bShouldSwapParticles);

		// Strip the Instanced wrapper if there is one
		if (const FImplicitObjectInstanced* InstancedA = ImplicitA->AsA<FImplicitObjectInstanced>())
		{
			ImplicitA = InstancedA->GetInnerObject().Get();
		}
		if (const FImplicitObjectInstanced* InstancedB = ImplicitB->AsA<FImplicitObjectInstanced>())
		{
			ImplicitB = InstancedB->GetInnerObject().Get();
		}

		// Create the constraint for this shape pair
		// NOTE: this just creates the object. The collision detection is done in ProcessNewConstraints
		if (!bShouldSwapParticles)
		{
			FindOrCreateConstraint(
				ParticleA, ImplicitA, LeafObjectIndexA, ShapeInstanceA, LevelSetParticlesA, RelativeTransformA,
				ParticleB, ImplicitB, LeafObjectIndexB, ShapeInstanceB, LevelSetParticlesB, RelativeTransformB,
				CullDistance, ShapePairType, Context.GetSettings().bAllowManifolds, Flags.bUseSweep, Context);
		}
		else
		{
			FindOrCreateConstraint(
				ParticleB, ImplicitB, LeafObjectIndexB, ShapeInstanceB, LevelSetParticlesB, RelativeTransformB,
				ParticleA, ImplicitA, LeafObjectIndexA, ShapeInstanceA, LevelSetParticlesA, RelativeTransformA,
				CullDistance, ShapePairType, Context.GetSettings().bAllowManifolds, Flags.bUseSweep, Context);
		}
	}

	bool FGenericParticlePairMidPhase::DoBoundsOverlap(
		const FImplicitObject* ImplicitA,
		const FRigidTransform3& ParticleWorldTransformA,
		const FRigidTransform3& ShapeRelativeTransformA,
		const FImplicitObject* ImplicitB,
		const FRigidTransform3& ParticleWorldTransformB,
		const FRigidTransform3& ShapeRelativeTransformB,
		const Private::FImplicitBoundsTestFlags BoundsTestFlags,
		const FRealSingle DistanceCheckSize,
		const FReal CullDistance)
	{
		const FRigidTransform3 ShapeWorldTransformA = ShapeRelativeTransformA * ParticleWorldTransformA;
		const FRigidTransform3 ShapeWorldTransformB = ShapeRelativeTransformB * ParticleWorldTransformB;

		// NOTE: only expand one bounds by cull distance
		const FAABB3 ShapeWorldBoundsA = ImplicitA->CalculateTransformedBounds(ShapeWorldTransformA).Thicken(CullDistance);
		const FAABB3 ShapeWorldBoundsB = ImplicitB->CalculateTransformedBounds(ShapeWorldTransformB);

		// World-space expanded bounds check
		if (BoundsTestFlags.bEnableAABBCheck)
		{
			if (!ShapeWorldBoundsA.Intersects(ShapeWorldBoundsB))
			{
				return false;
			}
		}

		// World-space sphere bounds check
		if (BoundsTestFlags.bEnableDistanceCheck && (DistanceCheckSize > FRealSingle(0)))
		{
			const FVec3 Separation = ShapeWorldBoundsA.GetCenter() - ShapeWorldBoundsB.GetCenter();
			const FReal SeparationSq = Separation.SizeSquared();
			const FReal CullDistanceSq = FMath::Square(CullDistance + FReal(DistanceCheckSize));
			if (SeparationSq > CullDistanceSq)
			{
				return false;
			}
		}

		FVec3f RelativeMovement = FVec3f(0);

		if (BoundsTestFlags.bEnableOBBCheck0 || BoundsTestFlags.bEnableOBBCheck1)
		{
			// OBB-AABB test in both directions. This is beneficial for shapes which do not fit their AABBs very well,
			// which includes boxes and other shapes that are not roughly spherical. It is especially beneficial when
			// one shape is long and thin (i.e., it does not fit an AABB well when the shape is rotated).
			if (BoundsTestFlags.bEnableOBBCheck0)
			{
				if (!ImplicitOverlapOBBToAABB(ImplicitA, ImplicitB, ShapeWorldTransformA, ShapeWorldTransformB, RelativeMovement, CullDistance))
				{
					return false;
				}
			}

			if (BoundsTestFlags.bEnableOBBCheck1)
			{
				if (!ImplicitOverlapOBBToAABB(ImplicitB, ImplicitA, ShapeWorldTransformB, ShapeWorldTransformA, RelativeMovement, CullDistance))
				{
					return false;
				}
			}
		}

		return true;
	}

	FPBDCollisionConstraint* FGenericParticlePairMidPhase::FindOrCreateConstraint(
		FGeometryParticleHandle* InParticle0,
		const FImplicitObject* InImplicit0,
		const int32 InImplicitId0,
		const FShapeInstance* InShape0,
		const FBVHParticles* InBVHParticles0,
		const FRigidTransform3& InShapeRelativeTransform0,
		FGeometryParticleHandle* InParticle1,
		const FImplicitObject* InImplicit1,
		const int32 InImplicitId1,
		const FShapeInstance* InShape1,
		const FBVHParticles* InBVHParticles1,
		const FRigidTransform3& InShapeRelativeTransform1,
		const FReal CullDistance,
		const EContactShapesType ShapePairType,
		const bool bUseManifold,
		const bool bEnableSweep,
		const FCollisionContext& Context)
	{
		// This is a callback from the low-level collision function. It should always be the same two particles, though the
		// shapes may be some children in the implicit hierarchy. The particles could be in the opposite order though, and
		// this will depend on the shape types involved. E.g., with two particles each with a sphere and a box in a union
		// would require up to two Sphere-Box contacts, with the particles in opposite orders.
#if !UE_BUILD_TEST && !UE_BUILD_SHIPPING
		const bool bIsCorrectParticles = ((InParticle0 == Particle0) && (InParticle1 == Particle1)) || ((InParticle0 == Particle1) && (InParticle1 == Particle0));
		if (!ensureMsgf(bIsCorrectParticles, TEXT("Attempt to use MidPhase for particles %d - %d with particles %d - %d"), Particle0->ParticleID().LocalID, Particle1->ParticleID().LocalID, InParticle0->ParticleID().LocalID, InParticle1->ParticleID().LocalID))
		{
			// We somehow received a callback for the wrong particle pair...this should not happen
			return nullptr;
		}
#endif

		// @todo(chaos): bParticlesInExpectedOrder is a temporary cpp-only fix for a key collision in FindConstraint.
		// When we have swapped the order of the particles, we need to make sure that the key uses the shapes
		// in the order that the particles are in the MidPhase. We can fix this more cleanly by passing the 
		// FParticlePairMidPhaseCollisionKey in from GenerateCollisionsImplicitLeafImplicitLeaf where we already
		// know if we swapped particle order or not.
		const bool bParticlesInExpectedOrder = (InParticle0 == Particle0) && (InParticle1 == Particle1);
		const FParticlePairMidPhaseCollisionKey CollisionKey = bParticlesInExpectedOrder ? FParticlePairMidPhaseCollisionKey(InImplicitId0, InImplicitId1) : FParticlePairMidPhaseCollisionKey(InImplicitId1, InImplicitId0);
		FPBDCollisionConstraint* Constraint = FindConstraint(CollisionKey);

		// Check that we have the collision constraint for the expected pair of implicit objects. The key is assigned by the 
		// midphase based on the indices of the geometry in each particle and should be unique as long as those indices do not change.
#if !UE_BUILD_TEST && !UE_BUILD_SHIPPING
		const bool bIsCorrectImplicits = (Constraint == nullptr) || ((Constraint->GetImplicit0() == InImplicit0) && (Constraint->GetImplicit1() == InImplicit1) && (Constraint->GetCollisionParticles0() == InBVHParticles0) && (Constraint->GetCollisionParticles1() == InBVHParticles1));
		if (!ensureMsgf(bIsCorrectImplicits, TEXT("Collision Constraint for geometry pair was created for a different pair. Collisions will be dropped.")))
		{
			// If we get here, the collision constraint is not for this pair of implicit objects, which is a bug.
			// This might be because we added/removed collision geometry to a particle without resetting the collisions. 
			// The fix is to call Evolution->InvalidateParticle() after a geometry change.
			return nullptr;
		}
#endif

		if (Constraint == nullptr)
		{
			// NOTE: Using InParticle0 and InParticle1 here because the order may be different to what we have stored
			const Private::FCollisionSortKey CollisionSortKey = Private::FCollisionSortKey(InParticle0, InImplicitId0, InParticle1, InImplicitId1);
			Constraint = CreateConstraint(
				InParticle0, InImplicit0, InShape0, InBVHParticles0, InShapeRelativeTransform0, 
				InParticle1, InImplicit1, InShape1, InBVHParticles1, InShapeRelativeTransform1, 
				CollisionKey, CollisionSortKey,
				CullDistance, ShapePairType, bUseManifold, Context);
		}
		check(!Constraint->IsEnabled());

		NewConstraints.Add(Constraint);

		return Constraint;
	}

	FPBDCollisionConstraint* FGenericParticlePairMidPhase::FindConstraint(const FParticlePairMidPhaseCollisionKey& CollisionKey)
	{
		FPBDCollisionConstraintPtr* PConstraint = Constraints.Find(CollisionKey.GetKey());
		if (PConstraint != nullptr)
		{
			return (*PConstraint).Get();
		}
		return nullptr;
	}

	FPBDCollisionConstraint* FGenericParticlePairMidPhase::CreateConstraint(
		FGeometryParticleHandle* InParticle0,
		const FImplicitObject* Implicit0,
		const FPerShapeData* InShape0,
		const FBVHParticles* BVHParticles0,
		const FRigidTransform3& ShapeRelativeTransform0,
		FGeometryParticleHandle* InParticle1,
		const FImplicitObject* Implicit1,
		const FPerShapeData* InShape1,
		const FBVHParticles* BVHParticles1,
		const FRigidTransform3& ShapeRelativeTransform1,
		const FParticlePairMidPhaseCollisionKey& CollisionKey,
		const Private::FCollisionSortKey& CollisionSortKey,
		const FReal CullDistance,
		const EContactShapesType ShapePairType,
		const bool bUseManifold,
		const FCollisionContext& Context)
	{
		PHYSICS_CSV_SCOPED_EXPENSIVE(PhysicsVerbose, NarrowPhase_CreateConstraint);

		FPBDCollisionConstraintPtr Constraint = Context.GetAllocator()->CreateConstraint(
			InParticle0, Implicit0, InShape0, BVHParticles0, ShapeRelativeTransform0,
			InParticle1, Implicit1, InShape1, BVHParticles1, ShapeRelativeTransform1,
			CullDistance, bUseManifold, ShapePairType);

		const int32 CurrentEpoch = Context.GetAllocator()->GetCurrentEpoch();
		Constraint->GetContainerCookie().MidPhase = this;
		Constraint->GetContainerCookie().bIsMultiShapePair = true;
		Constraint->GetContainerCookie().CreationEpoch = CurrentEpoch;

		Constraint->SetCollisionSortKey(CollisionSortKey);

		return Constraints.Add(CollisionKey.GetKey(), MoveTemp(Constraint)).Get();
	}

	void PrefetchConstraint(const TArray<FPBDCollisionConstraint*>& Constraints, const int32 ConstraintIndex)
	{
		if (ConstraintIndex < Constraints.Num())
		{
			FPlatformMisc::PrefetchBlock(Constraints[ConstraintIndex], sizeof(FPBDCollisionConstraint));
		}
	}

	int32 FGenericParticlePairMidPhase::ProcessNewConstraints(
		const FReal CullDistance,
		const FReal Dt,
		const FCollisionContext& Context)
	{
		CHAOS_MIDPHASE_SCOPE_CYCLE_TIMER(FGenericParticlePairMidPhase_ProcessNewConstraints);

		int32 NumActive = 0;
		const bool bUseCCDSweep = Flags.bIsCCDActive && Flags.bUseSweep;

		const int32 NumNewConstraints = NewConstraints.Num();
		for (int32 ConstraintIndex = 0; ConstraintIndex < NumNewConstraints; ++ConstraintIndex)
		{
			FPBDCollisionConstraint* Constraint = NewConstraints[ConstraintIndex];
			PrefetchConstraint(NewConstraints, ConstraintIndex + 1);

			// CCD may be temporarily disabled by the user (via a midphase modifier) or because we are moving slowly.
			Constraint->SetCCDEnabled(Flags.bIsCCDActive);
			Constraint->SetCCDSweepEnabled(bUseCCDSweep);
			if (bChaosCCDResetTOIFix)
			{
				Constraint->ResetCCDTimeOfImpact();
			}

			// NOTE: Probe constraints are always active and we run collision detection for them at the end opf the frame
			bool bIsActive = true;

			// We should test the original probe flag, because the modifier could have changed it at this point
			// Reseting flag will occur only when constraint will be activated.
			if (!Constraint->BoundsTestFlags.bIsProbe)
			{
				if (!bUseCCDSweep)
				{
					bIsActive = UpdateCollision(Constraint, CullDistance, Dt, Context);
				}
				else
				{
					bIsActive = UpdateCollisionCCD(Constraint, CullDistance, Dt, Context);
				}
			}

			if (bIsActive)
			{
				Context.GetAllocator()->ActivateConstraint(Constraint);
				++NumActive;
			}
		}

		NewConstraints.Reset();

		return NumActive;
	}

	bool FGenericParticlePairMidPhase::UpdateCollision(
		FPBDCollisionConstraint* Constraint,
		const FReal CullDistance,
		const FReal Dt,
		const FCollisionContext& Context)
	{
		// @todo(chaos): share this code with FSingleShapePairCollisionDetector

		const FRigidTransform3 ShapeWorldTransform0 = Constraint->GetShapeRelativeTransform0() * FConstGenericParticleHandle(Constraint->GetParticle0())->GetTransformPQ();
		const FRigidTransform3 ShapeWorldTransform1 = Constraint->GetShapeRelativeTransform1() * FConstGenericParticleHandle(Constraint->GetParticle1())->GetTransformPQ();
		const int32 CurrentEpoch = Context.GetAllocator()->GetCurrentEpoch();
		const int32 LastEpoch = CurrentEpoch - 1;
		const bool bWasUpdatedLastTick = (Constraint->GetContainerCookie().LastUsedEpoch >= LastEpoch);

		// Update the world shape transforms on the constraint (we cannot just give it the PerShapeData 
		// pointer because of Unions - see FMultiShapePairCollisionDetector)
		Constraint->SetShapeWorldTransforms(ShapeWorldTransform0, ShapeWorldTransform1);

		Constraint->SetCullDistance(CullDistance);

		// If the constraint was not used last frame, it needs to be reset, otherwise we will try to reuse
		if (!bWasUpdatedLastTick || (Constraint->GetManifoldPoints().Num() == 0))
		{
			// Clear all manifold data including saved contact data
			Constraint->ResetManifold();
		}

		bool bWasManifoldRestored = false;
		if (Context.GetSettings().bAllowManifoldReuse && bWasUpdatedLastTick && Constraint->GetCanRestoreManifold())
		{
			// Update the existing manifold. We can re-use as-is if none of the points have moved much and the bodies have not moved much
			// NOTE: this can succeed in "restoring" even if we have no manifold points
			// NOTE: this uses the transforms from SetLastShapeWorldTransforms, so we can only do this if we were updated last tick
			bWasManifoldRestored = Constraint->TryRestoreManifold();
		}
		else
		{
			// We are not trying to reuse manifold points, so reset them but leave stored data intact (for friction)
			Constraint->ResetActiveManifoldContacts();
		}

		if (!bWasManifoldRestored)
		{
			if (!Context.GetSettings().bDeferNarrowPhase)
			{
				Collisions::UpdateConstraint(*Constraint, Constraint->GetShapeWorldTransform0(), Constraint->GetShapeWorldTransform1(), Dt);
			}

			// We will be updating the manifold so update transforms used to check for movement in UpdateAndTryRestoreManifold on future ticks
			// NOTE: We call this after Collisions::UpdateConstraint because it may reset the manifold and reset the bCanRestoreManifold flag.
			// @todo(chaos): Collisions::UpdateConstraint does not need to reset the manifold - fix that
			Constraint->SetLastShapeWorldTransforms(ShapeWorldTransform0, ShapeWorldTransform1);
		}

		// If we have a valid contact, add it to the active list
		// We also add it to the active list if collision detection is deferred because the data will be filled in later and we
		// don't know in advance whether we will pass the Phi check (deferred narrow phase is used with RBAN)
		const bool bShouldActivate = (Constraint->GetPhi() <= CullDistance || Context.GetSettings().bDeferNarrowPhase);

		if (bShouldActivate)
		{
			Constraint->SetIsInitialContact(!bWasUpdatedLastTick);
		}

		return bShouldActivate;
	}


	bool FGenericParticlePairMidPhase::UpdateCollisionCCD(
		FPBDCollisionConstraint* Constraint,
		const FReal CullDistance,
		const FReal Dt,
		const FCollisionContext& Context)
	{
		// @todo(chaos): share this code with FSingleShapePairCollisionDetector

		check(Constraint->GetCCDSweepEnabled());

		const FRigidTransform3 ShapeWorldTransform0 = Constraint->GetShapeRelativeTransform0() * FConstGenericParticleHandle(Constraint->GetParticle0())->GetTransformPQ();
		const FRigidTransform3 ShapeWorldTransform1 = Constraint->GetShapeRelativeTransform1() * FConstGenericParticleHandle(Constraint->GetParticle1())->GetTransformPQ();

		// Update the world shape transforms on the constraint (we cannot just give it the PerShapeData 
		// pointer because of Unions - see FMultiShapePairCollisionDetector)
		// NOTE: these are not used by CCD which continuously moves the particles
		Constraint->SetShapeWorldTransforms(ShapeWorldTransform0, ShapeWorldTransform1);

		Constraint->SetCullDistance(CullDistance);
		Constraint->ResetManifold();
		Constraint->ResetActiveManifoldContacts();

		FConstGenericParticleHandle P0 = Constraint->GetParticle0();
		FConstGenericParticleHandle P1 = Constraint->GetParticle1();

		// We need the previous transform for the swept collision detector. It assumes that the current
		// transform has been set on the constraint. 
		// We assume that the particle's center of mass moved in a stright line and that it's rotation has 
		// not changed so we calculate the previous transform from the current one and the velocity.
		// NOTE: These are actor transforms, not CoM transforms
		// @todo(chaos): Pass both start and end transforms to the collision detector
		const FRigidTransform3 CCDParticleWorldTransform0 = FRigidTransform3(P0->P() - P0->V() * Dt, P0->Q());
		const FRigidTransform3 CCDParticleWorldTransform1 = FRigidTransform3(P1->P() - P1->V() * Dt, P1->Q());
		const FRigidTransform3 CCDShapeWorldTransform0 = Constraint->GetShapeRelativeTransform0() * CCDParticleWorldTransform0;
		const FRigidTransform3 CCDShapeWorldTransform1 = Constraint->GetShapeRelativeTransform1() * CCDParticleWorldTransform1;
		const bool bDidSweep = Collisions::UpdateConstraintSwept(*Constraint, CCDShapeWorldTransform0, CCDShapeWorldTransform1, Dt);

#if CHAOS_DEBUG_DRAW
		if (CVars::ChaosSolverDrawCCDInteractions)
		{
			if (FConstGenericParticleHandle(Constraint->GetParticle0())->CCDEnabled())
			{
				DebugDraw::DrawShape(CCDShapeWorldTransform0, Constraint->GetImplicit0(), Constraint->GetShape0(), FColor::Black, DebugDraw::ChaosDebugDrawCCDDuration, &CVars::ChaosSolverDebugDebugDrawSettings);
				DebugDraw::DrawShape(ShapeWorldTransform0, Constraint->GetImplicit0(), Constraint->GetShape0(), FColor::White, DebugDraw::ChaosDebugDrawCCDDuration, &CVars::ChaosSolverDebugDebugDrawSettings);
			}
			if (FConstGenericParticleHandle(Constraint->GetParticle1())->CCDEnabled())
			{
				DebugDraw::DrawShape(CCDShapeWorldTransform1, Constraint->GetImplicit1(), Constraint->GetShape1(), FColor::Black, DebugDraw::ChaosDebugDrawCCDDuration, &CVars::ChaosSolverDebugDebugDrawSettings);
				DebugDraw::DrawShape(ShapeWorldTransform1, Constraint->GetImplicit1(), Constraint->GetShape1(), FColor::White, DebugDraw::ChaosDebugDrawCCDDuration, &CVars::ChaosSolverDebugDebugDrawSettings);
			}
		}
#endif

		bool bShouldActivate = bDidSweep && (Constraint->GetCCDTimeOfImpact() < FReal(1));

		// If we did not get a sweep hit (TOI > 1) or did not sweep (bDidSweep = false), we need to run standard collision detection at T=1.
		// Likewise, if we did get a sweep hit but it's at TOI = 1, treat this constraint as a regular non-swept constraint and skip the rewind.
		// NOTE: The sweep will report TOI==1 for "shallow" sweep hits below the CCD thresholds in the constraint.
		if ((!bDidSweep) || (Constraint->GetCCDTimeOfImpact() >= FReal(1)))
		{
			Collisions::UpdateConstraint(*Constraint, Constraint->GetShapeWorldTransform0(), Constraint->GetShapeWorldTransform1(), Dt);
			Constraint->SetCCDSweepEnabled(false);
			bShouldActivate = Constraint->GetPhi() < CullDistance;
		}

		if (bShouldActivate)
		{
			const int32 CurrentEpoch = Context.GetAllocator()->GetCurrentEpoch();
			const int32 LastEpoch = CurrentEpoch - 1;
			const bool bWasUpdatedLastTick = IsUsedSince(LastEpoch);
			Constraint->SetIsInitialContact(!bWasUpdatedLastTick);
		}

		return bShouldActivate;
	}

	void FGenericParticlePairMidPhase::PruneConstraints(const int32 CurrentEpoch)
	{
		// Must call ProcessNewCollisions before Prune
		check(NewConstraints.Num() == 0);

		// Find all the expired collisions
		FMemMark Mark(FMemStack::Get());
		TArray<uint64, TMemStackAllocator<alignof(uint32)>> Pruned;
		Pruned.Reserve(Constraints.Num());

		for (TMap<uint64, FPBDCollisionConstraintPtr>::ElementType& KVP : Constraints)
		{
			const uint64 CollisionKey = KVP.Key;
			FPBDCollisionConstraintPtr& Constraint = KVP.Value;

			// NOTE: Constraints in sleeping islands should be kept alive. They will still be in the graph
			// and will be restored when the island wakes.
			if ((Constraint->GetContainerCookie().LastUsedEpoch < CurrentEpoch) && !Constraint->IsInConstraintGraph())
			{
				Pruned.Add(CollisionKey);
			}
		}

		// Destroy expired collisions
		for (uint64 CollisionKey : Pruned)
		{
			Constraints.Remove(CollisionKey);
		}
	}

	void FGenericParticlePairMidPhase::WakeCollisionsImpl(const int32 CurrentEpoch)
	{
		for (TMap<uint64, FPBDCollisionConstraintPtr>::ElementType& KVP : Constraints)
		{
			FPBDCollisionConstraintPtr& Constraint = KVP.Value;
			if (Constraint->GetContainerCookie().LastUsedEpoch >= LastUsedEpoch)
			{
				if (bChaosMidPhaseActivateWakingConstraints)
				{
					ReactivateCollisionConstraint(Constraint.Get());
				}
				else
				{
					Constraint->GetContainerCookie().LastUsedEpoch = CurrentEpoch;
					Constraint->GetContainerCookie().ConstraintIndex = INDEX_NONE;
					Constraint->GetContainerCookie().CCDConstraintIndex = INDEX_NONE;
				}
			}
		}
	}

	void FGenericParticlePairMidPhase::InjectCollisionImpl(const FPBDCollisionConstraint& Constraint, const FCollisionContext& Context)
	{
		// @todo(chaos): support rewind/resim here
#if !UE_BUILD_SHIPPING
		UE_LOGF(LogChaos, Warning, "Unsupported, handle Rewind/Resim in FGenericParticlePairMidPhase::InjectCollisionImpl");
#endif
	}


	////////////////////////////////////////////////////////////////////////////////////////
	////////////////////////////////////////////////////////////////////////////////////////
	////////////////////////////////////////////////////////////////////////////////////////
	////////////////////////////////////////////////////////////////////////////////////////


	FSphereApproximationParticlePairMidPhase::FSphereApproximationParticlePairMidPhase()
		: FParticlePairMidPhase(EParticlePairMidPhaseType::SphereApproximation)
		, Sphere0(FVec3(0), 0.0)
		, Sphere1(FVec3(0), 0.0)
		, SphereShape0(nullptr)
		, SphereShape1(nullptr)
		, LastUsedEpoch(INDEX_NONE)
		, bHasSpheres(false)
	{
	}

	void FSphereApproximationParticlePairMidPhase::ResetImpl()
	{
		Constraint.Reset();
	}

	void FSphereApproximationParticlePairMidPhase::BuildDetectorsImpl()
	{
		if (!IsValid())
		{
			return;
		}

		if (!GetParticle0()->HasBounds() || !GetParticle1()->HasBounds())
		{
			return;
		}

		const FShapeInstanceArray& Shapes0 = GetParticle0()->ShapeInstances();
		const FShapeInstanceArray& Shapes1 = GetParticle1()->ShapeInstances();
		if (Shapes0.IsEmpty() || Shapes1.IsEmpty())
		{
			return;
		}

		// See if we should collide (do any shape pairs collide)
		bool bDoCollide = false;
		for (int32 ShapeIndex0 = 0; ShapeIndex0 < Shapes0.Num(); ++ShapeIndex0)
		{
			const FShapeInstance* ShapeInstance0 = Shapes0[ShapeIndex0].Get();
			for (int32 ShapeIndex1 = 0; ShapeIndex1 < Shapes1.Num(); ++ShapeIndex1)
			{
				const FShapeInstance* ShapeInstance1 = Shapes1[ShapeIndex1].Get();

				const FImplicitObject* Implicit0 = ShapeInstance0->GetLeafGeometry();
				const EImplicitObjectType ImplicitType0 = Private::GetImplicitCollisionType(Particle0, Implicit0);

				const FImplicitObject* Implicit1 = ShapeInstance1->GetLeafGeometry();
				const EImplicitObjectType ImplicitType1 = Private::GetImplicitCollisionType(Particle1, Implicit1);

				// Use materials etc from the first overlapping shape pairs
				if (SphereShape0 == nullptr)
				{
					SphereShape0 = ShapeInstance0;
					SphereShape1 = ShapeInstance1;
				}

				const bool bDoPassFilter = ShapePairNarrowPhaseFilter(ImplicitType0, ShapeInstance0, ImplicitType1, ShapeInstance1);
				if (bDoPassFilter)
				{
					bDoCollide = true;
					break;
				}
			}
		}

		if (!bDoCollide)
		{
			return;
		}

		// Initialize the sphere centers and radii
		InitSphere(GetParticle0(), Sphere0);
		InitSphere(GetParticle1(), Sphere1);
		bHasSpheres = true;
	}

	void FSphereApproximationParticlePairMidPhase::InitSphere(
		const FGeometryParticleHandle* InParticle, 
		FImplicitSphere3& OutSphere)
	{
		// @todo(chaos): maybe we should only consider the bounds of the shapes that pass the ShapePairNarrowPhaseFilter?
		const FVec3 Center = InParticle->LocalBounds().Center();
		const FVec3 Extents = InParticle->LocalBounds().Extents();
		const FReal RadiusAlpha = FMath::Clamp(ChaosOneWayPairSphereRadiusAlpha, 0.0f, 1.0f);
		const FReal Radius = 0.5f * FMath::Lerp(Extents.GetMin(), Extents.GetMax(), RadiusAlpha);

		OutSphere = FImplicitSphere3(Center, Radius);
	}

	int32 FSphereApproximationParticlePairMidPhase::GenerateCollisionsImpl(
		const FRealSingle Dt,
		const FRealSingle CullDistance,
		const FVec3f& RelativeMovement,
		const FCollisionContext& Context)
	{
		if (!bHasSpheres)
		{
			return 0;
		}

		// NOTE: We are still using the bounds generated from the real collision shapes for the cull test
		const FAABB3& WorldBounds0 = GetParticle0()->WorldSpaceInflatedBounds();
		const FAABB3& WorldBounds1 = GetParticle1()->WorldSpaceInflatedBounds();
		if (!WorldBounds0.Intersects(WorldBounds1))
		{
			return 0;
		}

		const int32 CurrentEpoch = Context.GetAllocator()->GetCurrentEpoch();
		const int32 LastEpoch = CurrentEpoch - 1;
		const bool bWasUpdatedLastTick = IsUsedSince(LastEpoch);

		const FRigidTransform3 ShapeWorldTransform0 = FConstGenericParticleHandle(GetParticle0())->GetTransformPQ();
		const FRigidTransform3 ShapeWorldTransform1 = FConstGenericParticleHandle(GetParticle1())->GetTransformPQ();

		// Sphere distance culling
		const FVec3 SphereWorldPos0 = ShapeWorldTransform0.TransformPositionNoScale(FVec3(Sphere0.GetCenterf()));
		const FVec3 SphereWorldPos1 = ShapeWorldTransform1.TransformPositionNoScale(FVec3(Sphere1.GetCenterf()));
		const FVec3 DR = SphereWorldPos0 - SphereWorldPos1;
		const FReal DRLenSq = DR.SizeSquared();
		const FReal CullSeparation = Sphere0.GetRadiusf() + Sphere1.GetRadiusf() + CullDistance;
		if (DRLenSq > FMath::Square(CullSeparation))
		{
			return 0;
		}

		// Create and set up constraint
		if (!Constraint.IsValid())
		{
			Constraint = Context.GetAllocator()->CreateConstraint(
				GetParticle0(), &Sphere0, SphereShape0, nullptr, FRigidTransform3(), 
				GetParticle1(), &Sphere1, SphereShape1, nullptr, FRigidTransform3(),
				CullDistance, true, EContactShapesType::SphereSphere);

			Constraint->GetContainerCookie().MidPhase = this;
			Constraint->GetContainerCookie().bIsMultiShapePair = false;
			Constraint->GetContainerCookie().CreationEpoch = CurrentEpoch;
			Constraint->SetCollisionSortKey(Private::FCollisionSortKey(Particle0, 0, Particle1, 0));
		}

		Constraint->SetShapeWorldTransforms(ShapeWorldTransform0, ShapeWorldTransform1);
		Constraint->SetCullDistance(CullDistance);

		if (!bWasUpdatedLastTick || (Constraint->GetManifoldPoints().Num() == 0))
		{
			// Clear all manifold data including saved contact data
			Constraint->ResetManifold();
		}

		// We are not trying to reuse manifold points, so reset them but leave stored data intact (for friction)
		Constraint->ResetActiveManifoldContacts();

		// Sphere collision
		FContactPoint ContactPoint = SphereSphereContactPoint(Sphere0, ShapeWorldTransform0, Sphere1, ShapeWorldTransform1, CullDistance);
		Constraint->SetOneShotManifoldContacts({ ContactPoint });

		// Activate constraint if we have a contact
		if (Constraint->GetPhi() <= CullDistance)
		{
			Constraint->SetIsInitialContact(!bWasUpdatedLastTick);

			if (Context.GetAllocator()->ActivateConstraint(Constraint.Get()))
			{
				LastUsedEpoch = CurrentEpoch;
				return 1;
			}
		}

		return 0;
	}

	void FSphereApproximationParticlePairMidPhase::WakeCollisionsImpl(
		const int32 CurrentEpoch)
	{
		if (Constraint.IsValid() && (Constraint->GetContainerCookie().LastUsedEpoch >= LastUsedEpoch))
		{
			if (bChaosMidPhaseActivateWakingConstraints)
			{
				ReactivateCollisionConstraint(Constraint.Get());
			}
			else
			{
				Constraint->GetContainerCookie().LastUsedEpoch = CurrentEpoch;
				Constraint->GetContainerCookie().ConstraintIndex = INDEX_NONE;
				Constraint->GetContainerCookie().CCDConstraintIndex = INDEX_NONE;
			}
		}
	}

	void FSphereApproximationParticlePairMidPhase::InjectCollisionImpl(
		const FPBDCollisionConstraint& InConstraint, 
		const FCollisionContext& Context)
	{
		const int32 CurrentEpoch = Context.GetAllocator()->GetCurrentEpoch();

		if (!Constraint.IsValid())
		{
			Constraint = Context.GetAllocator()->CreateConstraint();
			Constraint->GetContainerCookie().MidPhase = this;
			Constraint->GetContainerCookie().bIsMultiShapePair = false;
			Constraint->GetContainerCookie().CreationEpoch = CurrentEpoch;
		}

		// Copy the constraint data over the existing one (ensure we do not replace data required by the graph and the allocator/container)
		Constraint->RestoreFrom(InConstraint);

		// Add the constraint to the active list
		// If the constraint already existed and was already active, this will do nothing
		Context.GetAllocator()->ActivateConstraint(Constraint.Get());
		LastUsedEpoch = CurrentEpoch;
	}
}


