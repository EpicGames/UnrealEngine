// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/Collision/SimRaycast.h"
#include "Chaos/CollisionFilterData.h"
#include "Chaos/ParticleHandle.h"

namespace Chaos::Private
{
	bool SimRaycastImplicit(const FSimRaycastRay& InRay, const FImplicitObject* InImplicit, const FRigidTransform3& InImplicitTransform, FSimRaycastHit& InOutHit)
	{
		// We assume the ray direction is normalized but that's fairly expensive to test...
		checkSlow(InRay.Dir.IsNormalized());

		// @todo(chaos): could optimize uniform scale rather than just unit scale...
		const bool bIsImplicitUnitScale = FVec3::IsNearlyEqual(InImplicitTransform.GetScale3D(), FVec3(1, 1, 1), UE_SMALL_NUMBER);

		// Transform the raycast into the space of the implicit
		FVec3 LocalStartPos;
		FVec3 LocalDir;
		FReal LocalLength;
		if (bIsImplicitUnitScale)
		{
			LocalStartPos = InImplicitTransform.InverseTransformPositionNoScale(InRay.StartPos);
			LocalDir = InImplicitTransform.InverseTransformVectorNoScale(InRay.Dir);
			LocalLength = InRay.Length;
		}
		else
		{
			const FVec3 LocalDelta = InImplicitTransform.InverseTransformVector(InRay.Dir * InRay.Length);
			LocalStartPos = InImplicitTransform.InverseTransformPosition(InRay.StartPos);
			LocalDir = FVec3::Zero();
			LocalLength = LocalDelta.Size();
			if (LocalLength > 0)
			{
				LocalDir = LocalDelta / LocalLength;
			}
		}

		FVec3 LocalHitPos, LocalHitNormal;
		FReal LocalHitDistance;
		int32 HitFaceIndex;
		if (InImplicit->Raycast(LocalStartPos, LocalDir, LocalLength, 0.0, LocalHitDistance, LocalHitPos, LocalHitNormal, HitFaceIndex))
		{
			// If we have an existing hit, ignore hits after it
			const FReal HitTOI = (LocalLength > 0) ? LocalHitDistance / LocalLength : 0;
			const bool bIsCloserHit = !InOutHit.IsHit() || (HitTOI < InOutHit.HitTOI);
			if (bIsCloserHit)
			{
				// Save output in world space
				if (bIsImplicitUnitScale)
				{
					InOutHit.HitPosition = InImplicitTransform.TransformPositionNoScale(LocalHitPos);
					InOutHit.HitNormal = InImplicitTransform.TransformVectorNoScale(LocalHitNormal);
					InOutHit.HitDistance = LocalHitDistance;
				}
				else
				{
					InOutHit.HitPosition = InImplicitTransform.TransformPosition(LocalHitPos);
					InOutHit.HitNormal = InImplicitTransform.TransformNormal(LocalHitNormal);
					InOutHit.HitDistance = HitTOI * InRay.Length;
				}

				// Hit normal and face normal always the same for raycasts.
				InOutHit.HitFaceNormal = InOutHit.HitNormal;

				// NOTE: Particle and Shape are unknown at this level, and must be filled in by the caller.
				InOutHit.HitParticle = nullptr;
				InOutHit.HitShape = nullptr;
				InOutHit.HitGeometry = InImplicit;
				InOutHit.HitFaceIndex = HitFaceIndex;
				InOutHit.HitTOI = HitTOI;

				return true;
			}
		}

		return false;
	}

	FSimQueryDataParticleFilter::FSimQueryDataParticleFilter(const Filter::FQueryFilterData& InFilterData)
		: FilterData(InFilterData)
	{
	}

	bool FSimQueryDataParticleFilter::ApplyFilter(const FGeometryParticleHandle* InParticleHandle) const
	{
		return true;
	}

	FSimQueryDataShapeFilter::FSimQueryDataShapeFilter(const Filter::FQueryFilterData& InFilterData, bool bInDetectSimple, bool bInDetectComplex)
		: FilterData(InFilterData)
	{
		Flags.bDetectSimple = bInDetectSimple;
		Flags.bDetectComplex = bInDetectComplex;
	}

	bool FSimQueryDataShapeFilter::ApplyFilter(const FPerShapeData* Shape, const FImplicitObject* Implicit) const
	{
		// Run the shape Simple/Complex filter
		const EImplicitObjectType ImplicitType = GetInnerType(Implicit->GetType());
		const bool bIsMesh = (ImplicitType == ImplicitObjectType::TriangleMesh);
		const bool bIsComplex = bIsMesh || (Shape->GetCollisionTraceType() == EChaosCollisionTraceFlag::Chaos_CTF_UseSimpleAsComplex);
		const bool bIsSimple = !bIsMesh || (Shape->GetCollisionTraceType() == EChaosCollisionTraceFlag::Chaos_CTF_UseComplexAsSimple);
		const bool bCanDetect = ((bIsComplex && !!Flags.bDetectComplex) || (bIsSimple && !!Flags.bDetectSimple));
		if (!bCanDetect)
		{
			return false;
		}

		// Run the channel filter
		// NOTE: We return true if we want to keep the result
		const Filter::ENarrowFilterResult HitType = FilterData.NarrowFilter(Shape->GetShapeFilterData());
		return HitType != Filter::ENarrowFilterResult::None;
	}

	bool SimRaycastBatchFirstHits(
		ISpatialAcceleration<FAccelerationStructureHandle, FReal, 3>* SpatialAcceleration,
		const TArrayView<const Private::FSimRaycastRay>& InRays,
		const Filter::FQueryFilterData& QueryFilterData,
		const bool bDetectSimple, const bool bDetectComplex,
		const TArrayView<Private::FSimRaycastHit>& OutHits)
	{
		if (!ensureMsgf(InRays.Num() == OutHits.Num(), TEXT("Mismatched input and output arrays")))
		{
			return 0;
		}

		bool bAnyHits = false;

		// Use default query filtering (but against shape SimFilter, not their QueryFilter)
		FSimQueryDataParticleFilter ParticleFilter = FSimQueryDataParticleFilter(QueryFilterData);
		FSimQueryDataShapeFilter ShapeFilter = FSimQueryDataShapeFilter(QueryFilterData, bDetectSimple, bDetectComplex);

		// Raycast against the overlapping particle shape
		const auto& RaycastParticleShape = 
			[&InRays, &OutHits, &bAnyHits](const FSimOverlapParticleShape& Overlap) -> void
			{
				const FRigidTransform3 ParticleTransform = FConstGenericParticleHandle(Overlap.HitParticle)->GetTransformPQ();

				const auto& RaycastLeafImplicit = 
					[&Overlap, &ParticleTransform, &InRays, &OutHits, &bAnyHits](const FImplicitObject* Implicit, const FRigidTransform3& ImplicitTransform, const int32 RootObjectIndex, const int32 ObjectIndex, const int32 LeafObjectIndex) -> void
					{
						// Calculate the world-space transform of the implicit object
						const FRigidTransform3 ParticleImplicitTransform = ImplicitTransform * ParticleTransform;

						// Cast all the rays against the ImplicitObject
						for (int32 RayIndex = 0; RayIndex < InRays.Num(); ++RayIndex)
						{
							if (SimRaycastImplicit(InRays[RayIndex], Implicit, ParticleImplicitTransform, OutHits[RayIndex]))
							{
								OutHits[RayIndex].HitParticle = Overlap.HitParticle;
								OutHits[RayIndex].HitShape = Overlap.HitShape;
								bAnyHits = true;
							}
						}
					};

				// Find all overlapping shapes and call the raycast lambda on them
				Overlap.HitShape->GetGeometry()->VisitLeafObjects(RaycastLeafImplicit);
			};

		// Calculate the query bounds (from all rays)
		FAABB3 QueryBounds = FAABB3::EmptyAABB();
		for (const FSimRaycastRay& Ray : InRays)
		{
			QueryBounds.GrowToInclude(Ray.StartPos);
			QueryBounds.GrowToInclude(Ray.StartPos + Ray.Length * Ray.Dir);
		}

		// Visit all shapes in the Spatial Acceleration that overlap any of the rays and run the raycasts
		SimOverlapBounds(SpatialAcceleration, QueryBounds, ParticleFilter, ShapeFilter, RaycastParticleShape);

		return bAnyHits;
	}

} // namespace Chaos::Private