// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/Collision/SimSweep.h"
#include "Chaos/ISpatialAcceleration.h"

namespace Chaos
{
	namespace Private
	{
		bool SimSweepParticleFirstHit(
			ISpatialAcceleration<FAccelerationStructureHandle, FReal, 3>* SpatialAcceleration,
			FIgnoreCollisionManager* InIgnoreCollisionManager,
			const FGeometryParticleHandle* SweptParticle,
			const FVec3& StartPos,
			const FRotation3& Rot,
			const FVec3& Dir,
			const FReal Length,
			FSimSweepParticleHit& OutHit,
			const FReal InHitDistanceEqualTolerance)
		{
			FSimSweepParticleFilterBroadPhase ParticleFilter(InIgnoreCollisionManager);
			FSimSweepShapeFilterNarrowPhase ShapeFilter;
			FSimSweepCollectorFirstHit HitCollector(InHitDistanceEqualTolerance, OutHit);

			SimSweepParticle(SpatialAcceleration, SweptParticle, StartPos, Rot, Dir, Length, ParticleFilter, ShapeFilter, HitCollector);

			return OutHit.IsHit();
		}

		bool SimOverlapBoundsAll(
			ISpatialAcceleration<FAccelerationStructureHandle, FReal, 3>* SpatialAcceleration,
			const FAABB3& QueryBounds,
			TArray<FSimOverlapParticleShape>& Overlaps)
		{
			// Accept all particles and shapes
			const auto& ParticleFilter = [](const FGeometryParticleHandle* Particle) -> bool { return true; };
			const auto& ShapeFilter = [](const FPerShapeData* Shape, const FImplicitObject* Implicit) { return true; };
			const auto& OverlapCollector = [&Overlaps](const FSimOverlapParticleShape& Overlap) { Overlaps.Add(Overlap); };
			const int32 InitialOverlapCount = Overlaps.Num();

			SimOverlapBounds(SpatialAcceleration, QueryBounds, ParticleFilter, ShapeFilter, OverlapCollector);

			return (Overlaps.Num() > InitialOverlapCount);
		}

	} // namespace Private
} // namespace Chaos