// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/Collision/SpatialAccelerationBroadPhase.h"

namespace Chaos
{
	namespace CVars
	{
		int32 NumWorkerCollisionFactor = 2;
		FAutoConsoleVariableRef CVarNumWorkerCollisionFactor(TEXT("p.Chaos.NumWorkerCollisionFactor"), NumWorkerCollisionFactor, TEXT("Set the number of tasks created for collision detection per worker."));
	}

	const TArray<UE::Tasks::FTask>& FSpatialAccelerationBroadPhase::GetBroadToNarrowTasks() const
	{
		return BroadToNarrowTasks;
	}

	void FSpatialAccelerationBroadPhase::ProduceOverlaps(
		FReal Dt,
		Private::FCollisionConstraintAllocator* Allocator,
		const FCollisionDetectorSettings& Settings,
		IResimCacheBase* ResimCache)
	{
		SCOPE_CYCLE_COUNTER(STAT_Collisions_BroadPhase);
		CSV_SCOPED_TIMING_STAT(PhysicsVerbose, DetectCollisions_BroadPhase);

		if (!ensure(SpatialAcceleration))
		{
			// Must call SetSpatialAcceleration
			return;
		}

		bNeedsResim = ResimCache && ResimCache->IsResimming();

		// Reset stats
		NumBroadPhasePairs = 0;
		NumMidPhases = 0;

		{
			SCOPE_CYCLE_COUNTER(STAT_Collisions_SpatialBroadPhase);

			if (const auto AABBTree = SpatialAcceleration->template As<TAABBTree<FAccelerationStructureHandle, TAABBTreeLeafArray<FAccelerationStructureHandle>>>())
			{
				ProduceOverlaps(Dt, *AABBTree, Allocator, Settings, ResimCache);
			}
			else if (const auto BV = SpatialAcceleration->template As<TBoundingVolume<FAccelerationStructureHandle>>())
			{
				ProduceOverlaps(Dt, *BV, Allocator, Settings, ResimCache);
			}
			else if (const auto AABBTreeBV = SpatialAcceleration->template As<TAABBTree<FAccelerationStructureHandle, TBoundingVolume<FAccelerationStructureHandle>>>())
			{
				ProduceOverlaps(Dt, *AABBTreeBV, Allocator, Settings, ResimCache);
			}
			else if (const auto Collection = SpatialAcceleration->template As<ISpatialAccelerationCollection<FAccelerationStructureHandle, FReal, 3>>())
			{
				Collection->PBDComputeConstraintsLowLevel(Dt, *this, Allocator, Settings, ResimCache);
			}
			else
			{
				check(false);  //question: do we want to support a dynamic dispatch version?
			}
		}
		{
			FChaosVDContextWrapper CVDContext;
			CVD_GET_WRAPPED_CURRENT_CONTEXT(CVDContext);

			const int32 NumContextTask = PendingTasks.Num();
			check(NumContextTask <= NumActiveBroadphaseContexts);
			// If bSingleWorkerPhysics is true, NumActiveBroadphaseContexts shouldn't bigger than 1
			check(!bSingleWorkerPhysics || NumActiveBroadphaseContexts <= 1);
			// If bSingleWorkerPhysics is true, PendingTasks should be empty
			check(!bSingleWorkerPhysics || PendingTasks.IsEmpty());
			if (bSingleWorkerPhysics)
			{
				AssignMidPhases(BroadphaseContexts[0]);
			}
			else
			{
				BroadToNarrowTasks.Reset();
				for (int32 ContextIndex = 0; ContextIndex < NumContextTask; ContextIndex++)
				{
					Private::FBroadPhaseContext& BroadPhaseContext = BroadphaseContexts[ContextIndex];

					UE::Tasks::FTask PendingTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, &BroadPhaseContext, CVDContext]()
						{
							CVD_SCOPE_CONTEXT(CVDContext.Context);
							AssignMidPhases(BroadPhaseContext);
						}, PendingTasks[ContextIndex]);
					BroadToNarrowTasks.Add(PendingTask);
				}
			}
		}

		// Run some error checks in non-shipping builds
		// NOTE: This must come after MidPhase assignment for now because that's where the filter is applied
		CheckOverlapResults();
	}

	void FSpatialAccelerationBroadPhase::ProduceCollisions(FReal Dt)
	{
		FChaosVDContextWrapper CVDContext;
		CVD_GET_WRAPPED_CURRENT_CONTEXT(CVDContext);

		// The broadphase overlaps probably resulted in a very different number of MidPhases 
		// (overlaps) on each thread. Redistribute the MidPhases so that we get more even 
		// processing. NOTE: This does not take into account that some MidPhases may be expensive. 
		// For that we would need to queue and redistribute the NarrowPhases, but currently each
		// MidPhase runs the NarrowPhase in ProcessMidPhase
		RedistributeMidPhasesInContexts();
		const int32 NumContextTask = PendingTasks.Num();
		PendingTasks.Reset();
		ContextsWithConstraints.Reset();
		check(NumContextTask <= NumActiveBroadphaseContexts);
		// If bSingleWorkerPhysics is true NumActiveBroadphaseContexts should be equal to 1
		check(!bSingleWorkerPhysics || NumActiveBroadphaseContexts == 1);
		if (bSingleWorkerPhysics)
		{
			ProcessMidPhases(Dt, BroadphaseContexts[0]);
		}
		else
		{
			for (int32 ContextIndex = 0; ContextIndex < NumContextTask; ContextIndex++)
			{
				Private::FBroadPhaseContext& BroadphaseContext = BroadphaseContexts[ContextIndex];
				UE::Tasks::FTask ProcessMidPhaseTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, &BroadphaseContext, Dt, CVDContext]()
					{
						CVD_SCOPE_CONTEXT(CVDContext.Context);
						ProcessMidPhases(Dt, BroadphaseContext);
					}, BroadToNarrowTasks[ContextIndex], UE::Tasks::ETaskPriority::High);
				PendingTasks.Add(ProcessMidPhaseTask);
				ContextsWithConstraints.Add(ContextIndex);
			}
		}
	}

	

	// Redistribute the midphases among the contexts to even out the per-core work a bit
	void FSpatialAccelerationBroadPhase::RedistributeMidPhasesInContexts()
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_Collisions_RedistributeMidPhases);

		if ((NumActiveBroadphaseContexts > 1) && CVars::bChaosMidPhaseRedistributionEnabled)
		{
			// We want this many midphases per worker
			const int32 NumMidPhasesPerContext = FMath::DivideAndRoundUp(NumMidPhases, NumActiveBroadphaseContexts);

			// Reserve array space
			for (int32 ContextIndex = 0; ContextIndex < NumActiveBroadphaseContexts; ++ContextIndex)
			{
				BroadphaseContexts[ContextIndex].MidPhases.Reserve(NumMidPhasesPerContext);
			}

			// Find the over-filled arrays and move elements to the under-filled arrays
			for (int32 SrcContextIndex = 0; SrcContextIndex < NumActiveBroadphaseContexts; ++SrcContextIndex)
			{
				TArray<FParticlePairMidPhase*>& SrcMidPhases = BroadphaseContexts[SrcContextIndex].MidPhases;
				int32 SrcNum = SrcMidPhases.Num();
				if (SrcNum > NumMidPhasesPerContext)
				{
					// SrcMidPhases is over-filled
					for (int32 DstContextIndex = 0; DstContextIndex < NumActiveBroadphaseContexts; ++DstContextIndex)
					{
						TArray<FParticlePairMidPhase*>& DstMidPhases = BroadphaseContexts[DstContextIndex].MidPhases;
						if ((DstContextIndex != SrcContextIndex) && (DstMidPhases.Num() < NumMidPhasesPerContext))
						{
							// DstMidPhases is under-filled, see how many elements we can move into it
							const int32 NumToMove = FMath::Min(SrcNum - NumMidPhasesPerContext, NumMidPhasesPerContext - DstMidPhases.Num());

							// Copy the elements from the end or Src to end of Dst (we will resize SrcMidPhases at the end)
							SrcNum -= NumToMove;
							DstMidPhases.Append(&SrcMidPhases[SrcNum], NumToMove);

							if (SrcNum == NumMidPhasesPerContext)
							{
								break;
							}
						}
					}

					check(SrcNum == NumMidPhasesPerContext);
					SrcMidPhases.SetNum(SrcNum);
				}
			}
		}
	}

	// Find or assign midphases to each of the overlapping particle pairs
	// @todo(chaos): optimize
	void FSpatialAccelerationBroadPhase::AssignMidPhases(Private::FBroadPhaseContext& BroadphaseContext)
	{
		SCOPE_CYCLE_COUNTER(STAT_Collisions_AssignMidPhases);

		Private::FCollisionContextAllocator* ContextAllocator = BroadphaseContext.CollisionContext.GetAllocator();

		BroadphaseContext.MidPhases.SetNum(BroadphaseContext.Overlaps.Num());

		int32 MidPhaseIndex = 0;
		for (int32 OverlapIndex = 0, OverlapEnd = BroadphaseContext.Overlaps.Num(); OverlapIndex < OverlapEnd; ++OverlapIndex)
		{
			Private::FBroadPhaseOverlap& Overlap = BroadphaseContext.Overlaps[OverlapIndex];

			// Check to see if the two particles are allowed to collide.
			// NOTE: this may also swap the order of the particles.
			Overlap.ApplyFilter(IgnoreCollisionManager, bNeedsResim);

			if (Overlap.bCollisionsEnabled)
			{
				if (CVars::ChaosOneWayInteractionPairCollisionMode == (int32)EOneWayInteractionPairCollisionMode::IgnoreCollision)
				{
					const bool bIsOneWay0 = FConstGenericParticleHandle(Overlap.Particles[0])->OneWayInteraction();
					const bool bIsOneWay1 = FConstGenericParticleHandle(Overlap.Particles[1])->OneWayInteraction();
					if (bIsOneWay0 && bIsOneWay1)
					{
						continue;
					}
				}

				// Get the midphase for this pair
				FParticlePairMidPhase* MidPhase = ContextAllocator->GetMidPhase(Overlap.Particles[0], Overlap.Particles[1], Overlap.Particles[Overlap.SearchParticleIndex], BroadphaseContext.CollisionContext);
				BroadphaseContext.MidPhases[MidPhaseIndex] = MidPhase;

				CVD_TRACE_MID_PHASE(MidPhase);

				++MidPhaseIndex;
			}
		}

		BroadphaseContext.MidPhases.SetNum(MidPhaseIndex);
	}

	// Process all the midphases: generate constraints and execute the narrowphase
	void FSpatialAccelerationBroadPhase::ProcessMidPhases(const FReal Dt, const Private::FBroadPhaseContext& BroadphaseContext)
	{
		SCOPE_CYCLE_COUNTER(STAT_Collisions_GenerateCollisions);

		// Prefetch initial set of MidPhases
		const int32 PrefetchLookahead = 4;
		const int32 NumContextMidPhases = BroadphaseContext.MidPhases.Num();
		for (int32 Index = 0; Index < NumContextMidPhases && Index < PrefetchLookahead; Index++)
		{
			BroadphaseContext.MidPhases[Index]->CachePrefetch();
		}

		for (int32 Index = 0; Index < NumContextMidPhases; Index++)
		{
			// Prefetch next MidPhase
			if (Index + PrefetchLookahead < NumContextMidPhases)
			{
				BroadphaseContext.MidPhases[Index + PrefetchLookahead]->CachePrefetch();
			}

			FParticlePairMidPhase* MidPhase = BroadphaseContext.MidPhases[Index];
			const FGeometryParticleHandle* Particle0 = MidPhase->GetParticle0();
			const FGeometryParticleHandle* Particle1 = MidPhase->GetParticle1();
			if ((Particle0 != nullptr) && (Particle1 != nullptr))
			{
				// Run MidPhase + NarrowPhase
				MidPhase->GenerateCollisions(BroadphaseContext.CollisionContext.GetSettings().BoundsExpansion, Dt, BroadphaseContext.CollisionContext);
			}

			CVD_TRACE_MID_PHASE(MidPhase);
		}

		PHYSICS_CSV_CUSTOM_EXPENSIVE(PhysicsCounters, NumFromBroadphase, NumPotentials, ECsvCustomStatOp::Accumulate);
	}

	void FSpatialAccelerationBroadPhase::CheckOverlapResults()
	{
#if CHAOS_CHECK_BROADPHASE
		// Make sure that no particle pair is in the overlap lists twice which will cause a race condition
		// in the narrow phase as multiple threads will attempt to build the same constraint(s) leading to
		// buffer overruns and who knows what other craziness.
		TSet<uint64> ParticlePairKeys;
		for (int32 ContextIndex = 0; ContextIndex < NumActiveBroadphaseContexts; ++ContextIndex)
		{
			for (const Private::FBroadPhaseOverlap& Overlap : BroadphaseContexts[ContextIndex].Overlaps)
			{
				if (Overlap.bCollisionsEnabled)
				{
					const Private::FCollisionParticlePairKey PairKey = Private::FCollisionParticlePairKey(Overlap.Particles[0], Overlap.Particles[1]);
					const bool bIsInSet = ParticlePairKeys.Contains(PairKey.GetKey());
					if (ensure(!bIsInSet))
					{
						ParticlePairKeys.Add(PairKey.GetKey());
					}
					else
					{
						// Last time this happened it was because a particle was in multiple views that should be mutually exclusive
						UE_LOGF(LogChaos, Fatal, "Broadphase overlaps generated the same particle pair twice");
					}
				}
			}
		}
#endif
	}
}