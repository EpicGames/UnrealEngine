// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/Collision/SphereConvexContactPoint.h"
#include "Chaos/Collision/ContactPoint.h"
#include "Chaos/Convex.h"
#include "Chaos/GJK.h"
#include "Chaos/GJKShape.h"
#include "Chaos/ImplicitObjectScaled.h"
#include "Chaos/Sphere.h"

namespace Chaos
{
	// Calculate the shortest vector for the point to depenetrate the convex.
	// Returns true unless there are no planes in the convex
	// Note: this may be called with small positive separations of order epsilon passed to GJK,
	// but the result is increasingly inaccurate as distance increases.
	template<typename T_CONVEX>
	bool ConvexPointPenetrationVector(const T_CONVEX& Convex, const FVec3& X, FVec3& OutNormal, FReal& OutPhi)
	{
		FReal MaxPhi = TNumericLimits<FReal>::Lowest();
		int32 MaxPlaneIndex = INDEX_NONE;

		const int32 NumPlanes = Convex.NumPlanes();
		for (int32 PlaneIndex = 0; PlaneIndex < NumPlanes; ++PlaneIndex)
		{
			const FReal Phi = Convex.GetPlane(PlaneIndex).SignedDistance(X);
			if (Phi > MaxPhi)
			{
				MaxPhi = Phi;
				MaxPlaneIndex = PlaneIndex;
			}
		}

		if (MaxPlaneIndex != INDEX_NONE)
		{
			OutPhi = MaxPhi;
			OutNormal = Convex.GetPlane(MaxPlaneIndex).Normal();
			return true;
		}

		return false;
	}


	// Use GJK (point to convex) to calculate separation.
	// Fall back to plane testing if penetrating by more than Radius.
	template<typename ConvexType>
	FContactPoint SphereConvexContactPointImpl(const FImplicitSphere3& Sphere, const ConvexType& Convex, const FRigidTransform3& SphereToConvexTransform)
	{
		if (Convex.NumPlanes() > 0)
		{
			FVec3 PosConvex, PosSphere, NormalConvex;
			FReal Phi;

			// Run GJK to find separating distance if available
			// NOTE: Sphere is treated as a point (its core shape), Convex margin is ignored so we are using the outer non-shrunken hull.
			// Sphere is also created in the space of the convex to eliminate the transform in the Support call (per GJK iteration).
			const FVec3 SpherePosConvex = SphereToConvexTransform.TransformPositionNoScale(FVec3(Sphere.GetCenterf()));
			const FReal SphereRadius = Sphere.GetRadiusf();

			// A point in the Minkowski Sum (A-B) in a likely direction
			// @todo(chaos): we could do better and return the correct direction separation vector most of the time by selecting the face 
			// most opposing the vector we have here (assuming we usually collide with a face rather than an edge). If we do this, 
			// it should probably only be done when the convex is larger than the sphere or the convex has few planes.
			const FVec3 InitialV = Convex.GetCenterOfMass() - SpherePosConvex;

			const EGJKDistanceResult GjkResult = GJKDistance(
				MakeGJKShape(Convex), 
				FGJKSphere(SpherePosConvex, 0), 
				InitialV, 
				Phi, PosConvex, PosSphere, NormalConvex);
				
			bool bHaveResult = (GjkResult != EGJKDistanceResult::DeepContact);

			// If the sphere center is inside the convex, find the depenetration vector
			if (!bHaveResult)
			{
				bHaveResult = ConvexPointPenetrationVector(Convex, SpherePosConvex, NormalConvex, Phi);
			}

			// Build the contact point
			if (bHaveResult)
			{
				// Results so far are all in convex-space and for a point rather than a sphere. Convert them.
				PosConvex = SpherePosConvex - Phi * NormalConvex;
				PosSphere = Sphere.GetCenterf() + SphereToConvexTransform.InverseTransformVectorNoScale(-SphereRadius * NormalConvex);
				Phi = Phi - SphereRadius;

				FContactPoint ContactPoint;
				ContactPoint.ShapeContactPoints[0] = PosSphere;
				ContactPoint.ShapeContactPoints[1] = PosConvex;
				ContactPoint.ShapeContactNormal = NormalConvex;
				ContactPoint.Phi = Phi;
				return ContactPoint;
			}
		}

		return FContactPoint();
	}

	FContactPoint SphereConvexContactPoint(const FImplicitSphere3& Sphere, const FImplicitObject& Object, const FRigidTransform3& SphereToConvexTransform)
	{
		if (const TImplicitObjectInstanced<FImplicitConvex3>* InstancedConvex = Object.template GetObject<const TImplicitObjectInstanced<FImplicitConvex3>>())
		{
			return SphereConvexContactPointImpl(Sphere, *InstancedConvex, SphereToConvexTransform);
		}
		else if (const TImplicitObjectScaled<FImplicitConvex3>* ScaledConvex = Object.template GetObject<const TImplicitObjectScaled<FImplicitConvex3>>())
		{
			return SphereConvexContactPointImpl(Sphere, *ScaledConvex, SphereToConvexTransform);
		}
		else if (const FImplicitConvex3* Convex = Object.template GetObject<const FImplicitConvex3>())
		{
			return SphereConvexContactPointImpl(Sphere, *Convex, SphereToConvexTransform);
		}
		return FContactPoint();
	}

	FContactPoint SphereConvexContactPoint(const FImplicitSphere3& Sphere, const FRigidTransform3& SphereTransform, const FImplicitObject& Object, const FRigidTransform3& ConvexTransform)
	{
		const FRigidTransform3 SphereToConvexTransform = SphereTransform.GetRelativeTransform(ConvexTransform);
		return SphereConvexContactPoint(Sphere, Object, SphereToConvexTransform);
	}

}
