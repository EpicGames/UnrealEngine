// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/Collision/SphereTriangleContactPoint.h"
#include "Chaos/Collision/ContactPoint.h"
#include "Chaos/Collision/ContactPointsMiscShapes.h"
#include "Chaos/Sphere.h"
#include "Chaos/Triangle.h"
#include "Chaos/Framework/UncheckedArray.h"

//PRAGMA_DISABLE_OPTIMIZATION

namespace Chaos
{
	void ConstructSphereTriangleOneShotManifold(const FImplicitSphere3& Sphere, const FTriangle& Triangle, const FReal CullDistance, FContactPointManifold& OutContactPoints)
	{
		// @todo(chaos): custom sphere-triangle contact point that avoids GJK

		FVec3 PosSphere, PosTri, NormalTri;
		FReal Phi;

		// A point in the Minkowski Sum (A-B) in a likely direction
		const FVec3 InitialV = Triangle.GetCentroid() - FVec3(Sphere.GetCenterf());

		// Run GJK to find separating distance if available
		// NOTE: Sphere is treated as a point (its core shape) for GJK to reduce iterations
		// Also, Sphere and Triangle are in the same space
		const EGJKDistanceResult GjkResult = GJKDistance(
			MakeGJKShape(Triangle),
			FGJKSphere(FVec3(Sphere.GetCenterf()), 0),
			InitialV,
			Phi, PosTri, PosSphere, NormalTri);

		const FReal SphereRadius = Sphere.GetRadiusf();

		// Build the contact point
		if ((Phi - SphereRadius) < CullDistance)
		{
			const int32 Index = OutContactPoints.Emplace(FContactPoint());
			OutContactPoints[Index].ShapeContactPoints[0] = PosSphere - SphereRadius * NormalTri;
			OutContactPoints[Index].ShapeContactPoints[1] = PosTri;
			OutContactPoints[Index].ShapeContactNormal = NormalTri;
			OutContactPoints[Index].Phi = Phi - SphereRadius;
		}
	}
}
