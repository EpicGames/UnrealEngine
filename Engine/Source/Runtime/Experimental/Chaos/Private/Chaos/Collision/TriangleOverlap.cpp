// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/Collision/TriangleOverlap.h"
#include "Chaos/Box.h"
#include "Chaos/ImplicitObjectScaled.h"
#include "Chaos/VectorUtility.h"

namespace Chaos
{
	bool ComputeCapsuleTriangleOverlapSimd(const VectorRegister4Float& A, const VectorRegister4Float& B, const VectorRegister4Float& C, const VectorRegister4Float& X1, const VectorRegister4Float& X2, FRealSingle Radius)
	{
		const VectorRegister4Float X2X1 = VectorSubtract(X1, X2);
		const VectorRegister4Float SqrX2X1 = VectorDot3(X2X1, X2X1);
		const VectorRegister4Float ZeroMask = VectorCompareEQ(VectorZeroFloat(), SqrX2X1);

		const VectorRegister4Float AB = VectorSubtract(B, A);
		const VectorRegister4Float BC = VectorSubtract(C, B);
		const VectorRegister4Float CA = VectorSubtract(A, C);

		const VectorRegister4Float Normal = VectorNormalize(VectorCross(AB, BC));

		constexpr FRealSingle ThirdFloat = 1.0f / 3.0f;
		constexpr VectorRegister4Float Third = MakeVectorRegisterFloatConstant(ThirdFloat, ThirdFloat, ThirdFloat, ThirdFloat);
		const VectorRegister4Float Centroid = VectorMultiply(VectorAdd(VectorAdd(A, B), C), Third);

		const VectorRegister4Float CentroidA = VectorSubtract(A, Centroid);
		const VectorRegister4Float CentroidB = VectorSubtract(B, Centroid);
		const VectorRegister4Float CentroidC = VectorSubtract(C, Centroid);

		const VectorRegister4Float AX1 = VectorSubtract(X1, A);
		const VectorRegister4Float AX2 = VectorSubtract(X2, A);
		const VectorRegister4Float BX1 = VectorSubtract(X1, B);
		const VectorRegister4Float CX1 = VectorSubtract(X1, C);

		// Edge plane normals and signed distances to each segment point
		// Mask to compare the distance of the triangle edge plane to the capsule radius.
		const VectorRegister4Float PlaneCA = VectorNormalize(VectorCross(CA, Normal));
		{
			FRealSingle EdgeX1 = VectorDot3Scalar(AX1, PlaneCA);
			FRealSingle EdgeX2 = VectorDot3Scalar(AX2, PlaneCA);
			FRealSingle Edge = FMath::Min<FRealSingle>(EdgeX1, EdgeX2);

			if (Edge > Radius)
			{
				return false;
			}
			// Make a check on another similar separating plane, but instead of using the triangle normal
			// use the capsule and the edge.
			// Make sure the other plane is on the right side of the triangle
			if (Edge > 0.0f)
			{
				VectorRegister4Float OtherNormal = VectorCross(AX1, CA);
				VectorRegister4Float OtherPlaneCA = VectorNormalize(VectorCross(CA, OtherNormal));
				EdgeX1 = VectorDot3Scalar(AX1, OtherPlaneCA);
				EdgeX2 = VectorDot3Scalar(AX2, OtherPlaneCA);
				Edge = FMath::Min<FRealSingle>(EdgeX1, EdgeX2);
				if (Edge > Radius)
				{
					return false;
				}

				OtherNormal = VectorCross(AX2, CA);
				OtherPlaneCA = VectorNormalize(VectorCross(CA, OtherNormal));
				EdgeX1 = VectorDot3Scalar(AX1, OtherPlaneCA);
				EdgeX2 = VectorDot3Scalar(AX2, OtherPlaneCA);
				Edge = FMath::Min<FRealSingle>(EdgeX1, EdgeX2);
				if (Edge > Radius)
				{
					return false;
				}
			}
		}
		const VectorRegister4Float PlaneAB = VectorNormalize(VectorCross(AB, Normal));
		{
			FRealSingle EdgeX1 = VectorDot3Scalar(BX1, PlaneAB);
			FRealSingle EdgeX2 = VectorDot3Scalar(VectorSubtract(X2, B), PlaneAB);
			FRealSingle Edge = FMath::Min<FRealSingle>(EdgeX1, EdgeX2);
			if (Edge > Radius)
			{
				return false;
			}

			if (Edge > 0.0f)
			{
				VectorRegister4Float OtherNormal = VectorCross(BX1, AB);
				VectorRegister4Float OtherPlaneAB = VectorNormalize(VectorCross(AB, OtherNormal));
				EdgeX1 = VectorDot3Scalar(BX1, OtherPlaneAB);
				EdgeX2 = VectorDot3Scalar(VectorSubtract(X2, B), OtherPlaneAB);
				Edge = FMath::Min<FRealSingle>(EdgeX1, EdgeX2);
				if (Edge > Radius)
				{
					return false;
				}

				OtherNormal = VectorCross(VectorSubtract(X2, B), AB);
				OtherPlaneAB = VectorNormalize(VectorCross(AB, OtherNormal));
				EdgeX1 = VectorDot3Scalar(BX1, OtherPlaneAB);
				EdgeX2 = VectorDot3Scalar(VectorSubtract(X2, B), OtherPlaneAB);
				Edge = FMath::Min<FRealSingle>(EdgeX1, EdgeX2);
				if (Edge > Radius)
				{
					return false;
				}
			}
		}
		const VectorRegister4Float PlaneBC = VectorNormalize(VectorCross(BC, Normal));
		{
			FRealSingle EdgeX1 = VectorDot3Scalar(CX1, PlaneBC);
			FRealSingle EdgeX2 = VectorDot3Scalar(VectorSubtract(X2, C), PlaneBC);
			FRealSingle Edge = FMath::Min<FRealSingle>(EdgeX1, EdgeX2);
			if (Edge > Radius)
			{
				return false;
			}

			if (Edge > 0.0f)
			{
				VectorRegister4Float OtherNormal = VectorCross(CX1, BC);
				VectorRegister4Float OtherPlaneBC = VectorNormalize(VectorCross(BC, OtherNormal));
				EdgeX1 = VectorDot3Scalar(CX1, OtherPlaneBC);
				EdgeX2 = VectorDot3Scalar(VectorSubtract(X2, C), OtherPlaneBC);
				Edge = FMath::Min<FRealSingle>(EdgeX1, EdgeX2);
				if (Edge > Radius)
				{
					return false;
				}

				OtherNormal = VectorCross(VectorSubtract(X2, C), BC);
				OtherPlaneBC = VectorNormalize(VectorCross(BC, OtherNormal));
				EdgeX1 = VectorDot3Scalar(CX1, OtherPlaneBC);
				EdgeX2 = VectorDot3Scalar(VectorSubtract(X2, C), OtherPlaneBC);
				Edge = FMath::Min<FRealSingle>(EdgeX1, EdgeX2);
				if (Edge > Radius)
				{
					return false;
				}
			}
		}

		// Plane Triangle
		FRealSingle AX1Dist = VectorDot3Scalar(AX1, Normal);
		FRealSingle AX2Dist = VectorDot3Scalar(AX2, Normal);
		if (FMath::Sign(AX1Dist) == FMath::Sign(AX2Dist))
		{
			FRealSingle ClosestDist = FMath::Min<FRealSingle>(FMath::Abs<FRealSingle>(AX1Dist), FMath::Abs<FRealSingle>(AX2Dist));
			if (ClosestDist > Radius)
			{
				return false;
			}
		}

		// Triangle Vertices
		const FRealSingle SqrRadius = Radius * Radius;
		{
			VectorRegister4Float TimeA = VectorClamp(VectorDivide(VectorDot3(X2X1, AX1), SqrX2X1), VectorZeroFloat(), VectorOneFloat());
			TimeA = VectorBitwiseNotAnd(ZeroMask, TimeA);
			const VectorRegister4Float PA = VectorMultiplyAdd(X1, VectorSubtract(VectorOneFloat(), TimeA), VectorMultiply(X2, TimeA));
			const VectorRegister4Float APA = VectorSubtract(PA, A);
			FRealSingle BetweenSeg = VectorDot3Scalar(VectorCross(PlaneAB, APA), VectorCross(PlaneCA, APA));
			if (BetweenSeg < 0.0f && VectorDot3Scalar(APA, CentroidA) > 0.0f)
			{
				if (VectorDot3Scalar(APA, APA) > SqrRadius)
				{
					return false;
				}
			}
		}
		{
			VectorRegister4Float TimeB = VectorClamp(VectorDivide(VectorDot3(X2X1, VectorSubtract(X1, B)), SqrX2X1), VectorZeroFloat(), VectorOneFloat());
			TimeB = VectorBitwiseNotAnd(ZeroMask, TimeB);
			const VectorRegister4Float PB = VectorMultiplyAdd(X1, VectorSubtract(VectorOneFloat(), TimeB), VectorMultiply(X2, TimeB));
			const VectorRegister4Float BPB = VectorSubtract(PB, B);
			FRealSingle BetweenSeg = VectorDot3Scalar(VectorCross(PlaneAB, BPB), VectorCross(PlaneBC, BPB));
			if (BetweenSeg < 0.0f && VectorDot3Scalar(BPB, CentroidB) > 0.0f)
			{
				if (VectorDot3Scalar(BPB, BPB) > SqrRadius)
				{
					return false;
				}
			}
		}
		{
			VectorRegister4Float TimeC = VectorClamp(VectorDivide(VectorDot3(X2X1, VectorSubtract(X1, C)), SqrX2X1), VectorZeroFloat(), VectorOneFloat());
			TimeC = VectorBitwiseNotAnd(ZeroMask, TimeC);
			const VectorRegister4Float PC = VectorMultiplyAdd(X1, VectorSubtract(VectorOneFloat(), TimeC), VectorMultiply(X2, TimeC));
			const VectorRegister4Float CPC = VectorSubtract(PC, C);
			FRealSingle BetweenSeg = VectorDot3Scalar(VectorCross(PlaneBC, CPC), VectorCross(PlaneCA, CPC));
			if (BetweenSeg < 0.0f && VectorDot3Scalar(CPC, CentroidC) > 0.0f)
			{
				if (VectorDot3Scalar(CPC, CPC) > SqrRadius)
				{
					return false;
				}
			}
		}

		// Edges
		{
			VectorRegister4Float EdgeSeparationAxis = VectorNormalize(VectorCross(X2X1, CA));
			EdgeSeparationAxis = VectorSelect(VectorCompareGT(VectorZeroFloat(), VectorDot3(CentroidA, EdgeSeparationAxis)), VectorNegate(EdgeSeparationAxis), EdgeSeparationAxis);

			const FRealSingle EdgeSeparationDist = VectorDot3Scalar(CX1, EdgeSeparationAxis);
			if (EdgeSeparationDist > Radius)
			{
				return false;
			}
		}
		{
			VectorRegister4Float EdgeSeparationAxis = VectorNormalize(VectorCross(X2X1, AB));
			EdgeSeparationAxis = VectorSelect(VectorCompareGT(VectorZeroFloat(), VectorDot3(CentroidB, EdgeSeparationAxis)), VectorNegate(EdgeSeparationAxis), EdgeSeparationAxis);

			const FRealSingle EdgeSeparationDist = VectorDot3Scalar(AX1, EdgeSeparationAxis);
			if (EdgeSeparationDist > Radius)
			{
				return false;
			}
		}
		{
			VectorRegister4Float EdgeSeparationAxis = VectorNormalize(VectorCross(X2X1, BC));
			EdgeSeparationAxis = VectorSelect(VectorCompareGT(VectorZeroFloat(), VectorDot3(CentroidC, EdgeSeparationAxis)), VectorNegate(EdgeSeparationAxis), EdgeSeparationAxis);

			const FRealSingle EdgeSeparationDist = VectorDot3Scalar(BX1, EdgeSeparationAxis);
			if (EdgeSeparationDist > Radius)
			{
				return false;
			}
		}

		return true;
	}

	bool ComputeSphereTriangleOverlapSimd(const VectorRegister4Float& A, const VectorRegister4Float& B, const VectorRegister4Float& C, const VectorRegister4Float& X, FRealSingle Radius)
	{
		const VectorRegister4Float AB = VectorSubtract(B, A);
		const VectorRegister4Float BC = VectorSubtract(C, B);
		const VectorRegister4Float CA = VectorSubtract(A, C);

		const VectorRegister4Float Normal = VectorNormalize(VectorCross(AB, BC));

		// Plane Triangle
		const VectorRegister4Float AX = VectorSubtract(X, A);
		const FRealSingle AXDist = VectorDot3Scalar(AX, Normal);

		if (FMath::Abs<FRealSingle>(AXDist) > Radius)
		{
			return false;
		}

		const VectorRegister4Float PlaneAB = VectorCross(AB, Normal);
		const VectorRegister4Float PlaneBC = VectorCross(BC, Normal);
		const VectorRegister4Float PlaneCA = VectorCross(CA, Normal);

		const FRealSingle SqrRadius = Radius * Radius;
		{
			const VectorRegister4Float SqrAB = VectorDot3(AB, AB);
			const VectorRegister4Float ZeroMask = VectorCompareEQ(VectorZeroFloat(), SqrAB);
			VectorRegister4Float Time = VectorClamp(VectorDivide(VectorDot3(AB, AX), SqrAB), VectorZeroFloat(), VectorOneFloat());
			Time = VectorBitwiseNotAnd(ZeroMask, Time);
			const VectorRegister4Float P = VectorMultiplyAdd(A, VectorSubtract(VectorOneFloat(), Time), VectorMultiply(B, Time));
			const VectorRegister4Float PX = VectorSubtract(X, P);

			const FRealSingle Dist = VectorDot3Scalar(PX, PlaneAB);
			if (Dist > 0.0f)
			{
				if (VectorDot3Scalar(PX, PX) > SqrRadius)
				{
					return false;
				}
			}
		}
		{
			const VectorRegister4Float SqrBC = VectorDot3(BC, BC);
			const VectorRegister4Float ZeroMask = VectorCompareEQ(VectorZeroFloat(), SqrBC);
			const VectorRegister4Float BX = VectorSubtract(X, B);
			VectorRegister4Float Time = VectorClamp(VectorDivide(VectorDot3(BC, BX), SqrBC), VectorZeroFloat(), VectorOneFloat());
			Time = VectorBitwiseNotAnd(ZeroMask, Time);
			const VectorRegister4Float P = VectorMultiplyAdd(B, VectorSubtract(VectorOneFloat(), Time), VectorMultiply(C, Time));
			const VectorRegister4Float PX = VectorSubtract(X, P);
			const FRealSingle Dist = VectorDot3Scalar(PX, PlaneBC);
			if (Dist > 0.0f)
			{
				if (VectorDot3Scalar(PX, PX) > SqrRadius)
				{
					return false;
				}
			}
		}
		{
			const VectorRegister4Float SqrCA = VectorDot3(CA, CA);
			const VectorRegister4Float ZeroMask = VectorCompareEQ(VectorZeroFloat(), SqrCA);
			const VectorRegister4Float CX = VectorSubtract(X, C);
			VectorRegister4Float Time = VectorClamp(VectorDivide(VectorDot3(CA, CX), SqrCA), VectorZeroFloat(), VectorOneFloat());
			Time = VectorBitwiseNotAnd(ZeroMask, Time);
			const VectorRegister4Float P = VectorMultiplyAdd(C, VectorSubtract(VectorOneFloat(), Time), VectorMultiply(A, Time));
			const VectorRegister4Float PX = VectorSubtract(X, P);
			const FRealSingle Dist = VectorDot3Scalar(PX, PlaneCA);
			if (Dist > 0.0f)
			{
				if (VectorDot3Scalar(PX, PX) > SqrRadius)
				{
					return false;
				}
			}
		}
		return true;
	}

	FBoxSimd::FBoxSimd()
	{
	}

	FBoxSimd::FBoxSimd(const FRigidTransform3& WorldScaleQueryTM, const TBox<FReal, 3>& InQueryGeom)
	{
		FRigidTransform3 Transform(WorldScaleQueryTM);
		Transform.SetTranslation(Transform.TransformPositionNoScale(InQueryGeom.GetCenter()));
		FVec3f HalfExtentsf = InQueryGeom.Extents() * 0.5;
		Initialize(Transform, HalfExtentsf);
	}

	FBoxSimd::FBoxSimd(const FRigidTransform3& QueryTM, const TImplicitObjectScaled< TBox<FReal, 3> >& QueryGeom)
	{
		FRigidTransform3 Transform(QueryTM);
		Transform.SetTranslation(Transform.TransformPositionNoScale(QueryGeom.GetUnscaledObject()->GetCenter() * QueryGeom.GetScale()));
		FVec3f HalfExtentsf = QueryGeom.GetUnscaledObject()->Extents() * 0.5 * (QueryGeom.GetScale() + UE_SMALL_NUMBER);
		Initialize(Transform, HalfExtentsf);
	}

	FORCEINLINE void FBoxSimd::Initialize(const FRigidTransform3& Transform, const FVec3f& HalfExtentsf)
	{
		TRotation<FRealSingle, 3> Rotation = Transform.GetRotation();
		PMatrix<FRealSingle, 3, 3> Matrix = Rotation.ToMatrix();
		XAxis = VectorLoadFloat3(Matrix.M[0]);
		YAxis = VectorLoadFloat3(Matrix.M[1]);
		ZAxis = VectorLoadFloat3(Matrix.M[2]);

		FVec3 Translation = Transform.GetTranslation();
		Position = MakeVectorRegisterFloatFromDouble(VectorLoadDouble3(&Translation.X));

		const VectorRegister4Float HalfExtents = VectorLoadFloat3(&HalfExtentsf.X);
		XHalfExtent = VectorReplicate(HalfExtents, 0);
		YHalfExtent = VectorReplicate(HalfExtents, 1);
		ZHalfExtent = VectorReplicate(HalfExtents, 2);

		XAxisHalfExtent = VectorMultiply(XAxis, XHalfExtent);
		YAxisHalfExtent = VectorMultiply(YAxis, YHalfExtent);
		ZAxisHalfExtent = VectorMultiply(ZAxis, ZHalfExtent);

		const VectorRegister4Float PosPoint = VectorAdd(VectorAdd(XAxisHalfExtent, YAxisHalfExtent), ZAxisHalfExtent);

		VectorRegister4Float OtherPoints[3];
		OtherPoints[0] = VectorAdd(VectorAdd(VectorNegate(XAxisHalfExtent), YAxisHalfExtent), ZAxisHalfExtent);
		OtherPoints[1] = VectorAdd(VectorAdd(XAxisHalfExtent, VectorNegate(YAxisHalfExtent)), ZAxisHalfExtent);
		OtherPoints[2] = VectorAdd(VectorAdd(XAxisHalfExtent, YAxisHalfExtent), VectorNegate(ZAxisHalfExtent));

		for (int32 i = 0; i < 3; i++)
		{
			Edges[i] = VectorSubtract(OtherPoints[i], PosPoint);
		}

	}

	FORCEINLINE_DEBUGGABLE bool FBoxSimd::ComputeEdge(const VectorRegister4Float& Normal, const VectorRegister4Float& TriangleVertex, const VectorRegister4Float& Edge, const VectorRegister4Float& Centroid) const
	{
		const VectorRegister4Float PlaneNormal = VectorNormalize(VectorCross(Edge, Normal));
		VectorRegister4Float XComp = VectorSelect(VectorCompareGT(VectorDot3(PlaneNormal, XAxisHalfExtent), VectorZeroFloat()), VectorNegate(XAxisHalfExtent), XAxisHalfExtent);
		VectorRegister4Float YComp = VectorSelect(VectorCompareGT(VectorDot3(PlaneNormal, YAxisHalfExtent), VectorZeroFloat()), VectorNegate(YAxisHalfExtent), YAxisHalfExtent);
		VectorRegister4Float ZComp = VectorSelect(VectorCompareGT(VectorDot3(PlaneNormal, ZAxisHalfExtent), VectorZeroFloat()), VectorNegate(ZAxisHalfExtent), ZAxisHalfExtent);

		const VectorRegister4Float LocalClosest = VectorAdd(VectorAdd(XComp, YComp), ZComp);
		const VectorRegister4Float ClosestPoint = VectorAdd(LocalClosest, Position);

		FRealSingle DistClos = VectorDot3Scalar(VectorSubtract(ClosestPoint, TriangleVertex), PlaneNormal);
		if (DistClos > 0.0)
		{
			return false;
		}

		// Triangle edge vs box edges 
		for (int32 i = 0; i < EdgeNum; i++)
		{
			VectorRegister4Float Axis = VectorCross(Edges[i], Edge);
			const VectorRegister4Float Sign = VectorDot3(VectorSubtract(TriangleVertex, Centroid), Axis);
			Axis = VectorSelect(VectorCompareLT(Sign, VectorZeroFloat()), VectorNegate(Axis), Axis);

			const VectorRegister4Float XCompCurrent = VectorSelect(VectorCompareGT(VectorDot3(Axis, XAxisHalfExtent), VectorZeroFloat()), VectorNegate(XAxisHalfExtent), XAxisHalfExtent);
			const VectorRegister4Float YCompCurrent = VectorSelect(VectorCompareGT(VectorDot3(Axis, YAxisHalfExtent), VectorZeroFloat()), VectorNegate(YAxisHalfExtent), YAxisHalfExtent);
			const VectorRegister4Float ZCompCurrent = VectorSelect(VectorCompareGT(VectorDot3(Axis, ZAxisHalfExtent), VectorZeroFloat()), VectorNegate(ZAxisHalfExtent), ZAxisHalfExtent);

			VectorRegister4Float LocalBoxVertex = VectorAdd(VectorAdd(XCompCurrent, YCompCurrent), ZCompCurrent);
			const VectorRegister4Float BoxVertex = VectorAdd(LocalBoxVertex, Position);

			const VectorRegister4Float SepVector = VectorSubtract(BoxVertex, TriangleVertex);
			FRealSingle SeparationMin = VectorDot3Scalar(SepVector, Axis);

			if (SeparationMin > UE_KINDA_SMALL_NUMBER)
			{
				return false;
			}
		}
		return true;
	}

	FORCEINLINE_DEBUGGABLE bool FBoxSimd::ComputeBoxPlane(const VectorRegister4Float& DistCenter, const VectorRegister4Float& AxisHalfExtent) const
	{
		const VectorRegister4Float SignMask = GlobalVectorConstants::SignMask();
		VectorRegister4Float AllPosSignVector = VectorBitwiseAnd(DistCenter, SignMask);
		VectorRegister4Float AllNegSignVector = VectorNegate(AllPosSignVector);

		constexpr static int32 TrueMask = 0b1111;
		if (VectorMaskBits(VectorCompareEQ(DistCenter, AllPosSignVector)) == TrueMask || VectorMaskBits(VectorCompareEQ(DistCenter, AllNegSignVector)) == TrueMask)
		{
			VectorRegister4Float IsSepX = VectorCompareGT(VectorAbs(DistCenter), AxisHalfExtent);
			if (VectorMaskBits(IsSepX) == TrueMask)
			{
				return false;
			}
		}
		return true;
	}

	bool FBoxSimd::OverlapTriangle(const VectorRegister4Float& A, const VectorRegister4Float& B, const VectorRegister4Float& C) const
	{
		const VectorRegister4Float PA = VectorSubtract(A, Position);
		const VectorRegister4Float PB = VectorSubtract(B, Position);
		const VectorRegister4Float PC = VectorSubtract(C, Position);

		// Box planes
		const VectorRegister4Float DistACenterX = VectorDot3(XAxis, PA);
		const VectorRegister4Float DistBCenterX = VectorDot3(XAxis, PB);
		const VectorRegister4Float DistCCenterX = VectorDot3(XAxis, PC);

		VectorRegister4Float DistCenterX = VectorUnpackLo(DistACenterX, DistBCenterX);
		DistCenterX = VectorMoveLh(DistCenterX, DistCCenterX);

		if (!ComputeBoxPlane(DistCenterX, XHalfExtent))
		{
			return false;
		}

		const VectorRegister4Float DistACenterY = VectorDot3(YAxis, PA);
		const VectorRegister4Float DistBCenterY = VectorDot3(YAxis, PB);
		const VectorRegister4Float DistCCenterY = VectorDot3(YAxis, PC);

		VectorRegister4Float DistCenterY = VectorUnpackLo(DistACenterY, DistBCenterY);
		DistCenterY = VectorMoveLh(DistCenterY, DistCCenterY);

		if (!ComputeBoxPlane(DistCenterY, YHalfExtent))
		{
			return false;
		}

		const VectorRegister4Float DistACenterZ = VectorDot3(ZAxis, PA);
		const VectorRegister4Float DistBCenterZ = VectorDot3(ZAxis, PB);
		const VectorRegister4Float DistCCenterZ = VectorDot3(ZAxis, PC);

		VectorRegister4Float DistCenterZ = VectorUnpackLo(DistACenterZ, DistBCenterZ);
		DistCenterZ = VectorMoveLh(DistCenterZ, DistCCenterZ);

		if (!ComputeBoxPlane(DistCenterZ, ZHalfExtent))
		{
			return false;
		}

		// Triangles Edges
		const VectorRegister4Float AB = VectorSubtract(B, A);
		const VectorRegister4Float BC = VectorSubtract(C, B);
		const VectorRegister4Float CA = VectorSubtract(A, C);

		const VectorRegister4Float Normal = VectorNormalize(VectorCross(AB, BC));
		constexpr FRealSingle ThirdFloat = 1.0f / 3.0f;
		constexpr VectorRegister4Float Third = MakeVectorRegisterFloatConstant(ThirdFloat, ThirdFloat, ThirdFloat, ThirdFloat);
		const VectorRegister4Float Centroid = VectorMultiply(VectorAdd(VectorAdd(A, B), C), Third);

		if (!ComputeEdge(Normal, A, AB, Centroid))
		{
			return false;
		}
		if (!ComputeEdge(Normal, B, BC, Centroid))
		{
			return false;
		}
		if (!ComputeEdge(Normal, C, CA, Centroid))
		{
			return false;
		}

		// Triangle Plane 
		VectorRegister4Float XCompNorm = VectorMultiply(XAxis, XHalfExtent);
		XCompNorm = VectorSelect(VectorCompareGT(VectorDot3(Normal, XCompNorm), VectorZeroFloat()), XCompNorm, VectorNegate(XCompNorm));
		VectorRegister4Float YCompNorm = VectorMultiply(YAxis, YHalfExtent);
		YCompNorm = VectorSelect(VectorCompareGT(VectorDot3(Normal, YCompNorm), VectorZeroFloat()), YCompNorm, VectorNegate(YCompNorm));
		VectorRegister4Float ZCompNorm = VectorMultiply(ZAxis, ZHalfExtent);
		ZCompNorm = VectorSelect(VectorCompareGT(VectorDot3(Normal, ZCompNorm), VectorZeroFloat()), ZCompNorm, VectorNegate(ZCompNorm));

		const VectorRegister4Float LoacalClosestNorm = VectorAdd(VectorAdd(XCompNorm, YCompNorm), ZCompNorm);
		const VectorRegister4Float ClosestPointNorm = VectorAdd(LoacalClosestNorm, Position);
		const VectorRegister4Float FurthestPointNorm = VectorSubtract(Position, LoacalClosestNorm);

		VectorRegister4Float ClosCent = VectorSubtract(ClosestPointNorm, Centroid);
		VectorRegister4Float FurtCent = VectorSubtract(FurthestPointNorm, Centroid);

		FRealSingle ClosestDist = VectorDot3Scalar(Normal, ClosCent);
		FRealSingle FurthestDist = VectorDot3Scalar(Normal, FurtCent);

		return !(FMath::Abs<FRealSingle>(ClosestDist) > UE_KINDA_SMALL_NUMBER &&
			FMath::Abs<FRealSingle>(FurthestDist) > UE_KINDA_SMALL_NUMBER &&
			FMath::Sign(ClosestDist) == FMath::Sign(FurthestDist));
	}

	const VectorRegister4Float FAABBSimd::SignBit = GlobalVectorConstants::SignBit();
	const VectorRegister4Float FAABBSimd::SignX = MakeVectorRegisterFloatMask(0x80000000, 0x00000000, 0x00000000, 0x00000000);
	const VectorRegister4Float FAABBSimd::SignY = MakeVectorRegisterFloatMask(0x00000000, 0x80000000, 0x00000000, 0x00000000);
	const VectorRegister4Float FAABBSimd::SignZ = MakeVectorRegisterFloatMask(0x00000000, 0x00000000, 0x80000000, 0x00000000);


	FAABBSimd::FAABBSimd()
		: Position(VectorZeroFloat())
	{
	}

	FAABBSimd::FAABBSimd(const TBox<FReal, 3>& QueryGeom)
	{
		FVec3f Positionf = QueryGeom.GetCenter();
		Position = VectorLoadFloat3(&Positionf.X);
		FVec3f HalfExtentsf = QueryGeom.Extents() * 0.5;
		HalfExtents = VectorLoadFloat3(&HalfExtentsf.X);
	}

	FAABBSimd::FAABBSimd(const TImplicitObjectScaled< TBox<FReal, 3> >& QueryGeom)
	{
		FVec3f Positionf = QueryGeom.GetUnscaledObject()->GetCenter() * QueryGeom.GetScale();
		Position = VectorLoadFloat3(&Positionf.X);
		FVec3f HalfExtentsf = QueryGeom.GetUnscaledObject()->Extents() * 0.5 * (QueryGeom.GetScale() + UE_SMALL_NUMBER);
		HalfExtents = VectorLoadFloat3(&HalfExtentsf.X);
	}

	FAABBSimd::FAABBSimd(const FVec3& Translation, const TBox<FReal, 3>& InQueryGeom)
	{
		FVec3f HalfExtentsf = InQueryGeom.Extents() * 0.5;
		Initialize(Translation + InQueryGeom.GetCenter(), HalfExtentsf);
	}

	FAABBSimd::FAABBSimd(const FVec3& Translation, const TImplicitObjectScaled< TBox<FReal, 3> >& QueryGeom)
	{
		FVec3f HalfExtentsf = QueryGeom.GetUnscaledObject()->Extents() * 0.5 * (QueryGeom.GetScale() + UE_SMALL_NUMBER);
		Initialize(Translation + QueryGeom.GetUnscaledObject()->GetCenter() * QueryGeom.GetScale(), HalfExtentsf);
	}

	FORCEINLINE void FAABBSimd::Initialize(const FVec3& Translation, const FVec3f& HalfExtentsf)
	{
		Position = MakeVectorRegisterFloatFromDouble(VectorLoadDouble3(&Translation.X));
		HalfExtents = VectorLoadFloat3(&HalfExtentsf.X);
	}

	FORCEINLINE_DEBUGGABLE bool FAABBSimd::ComputeEdgeOverlap(const VectorRegister4Float& TriangleEdge, const VectorRegister4Float& TriangleVertex, const VectorRegister4Float& Centroid) const
	{
		for (int32 i = 0; i < 3; i++)
		{
			VectorRegister4Float Axis = VectorCross(Edges[i], TriangleEdge);
			const VectorRegister4Float Sign = VectorDot3(VectorSubtract(TriangleVertex, Centroid), Axis);
			Axis = VectorSelect(VectorCompareLT(Sign, VectorZeroFloat()), VectorBitwiseXor(SignBit, Axis), Axis);

			// Take the opposite (xor) vector and extract sign (and), result in a (not and)
			const VectorRegister4Float AxisSigns = VectorBitwiseNotAnd(Axis, SignBit);
			const VectorRegister4Float LocalClosest = VectorBitwiseOr(AxisSigns, HalfExtents);
			const VectorRegister4Float ClosestPoint = VectorAdd(LocalClosest, Position);

			const FRealSingle ScaledSeparation = VectorDot3Scalar(VectorSubtract(ClosestPoint, TriangleVertex), Axis);
			if (ScaledSeparation > UE_KINDA_SMALL_NUMBER)
			{
				return false;
			}
		}
		return true;
	}

	bool FAABBSimd::OverlapTriangle(const VectorRegister4Float& A, const VectorRegister4Float& B, const VectorRegister4Float& C) const
	{
		const VectorRegister4Float PA = VectorSubtract(A, Position);
		const VectorRegister4Float PB = VectorSubtract(B, Position);
		const VectorRegister4Float PC = VectorSubtract(C, Position);

		// AABB planes
		VectorRegister4Float MinAbsDist = VectorMin(VectorMin(VectorAbs(PA), VectorAbs(PB)), VectorAbs(PC));

		const VectorRegister4Float MinDist = VectorMin(VectorMin(PA, PB), PC);
		const VectorRegister4Float MaxDist = VectorMax(VectorMax(PA, PB), PC);
		const VectorRegister4Float MinSign = VectorCastIntToFloat(VectorShiftRightImmArithmetic(VectorCastFloatToInt(VectorBitwiseAnd(SignBit, MinDist)), 1));
		const VectorRegister4Float MaxSign = VectorCastIntToFloat(VectorShiftRightImmArithmetic(VectorCastFloatToInt(VectorBitwiseAnd(SignBit, MaxDist)), 1));

		if (VectorMaskBits(VectorBitwiseAnd(VectorCompareGT(MinAbsDist, HalfExtents), VectorCompareEQ(MinSign, MaxSign))) != 0)
		{
			return false;
		}

		// Triangles Edges
		const VectorRegister4Float AB = VectorSubtract(B, A);
		const VectorRegister4Float BC = VectorSubtract(C, B);
		const VectorRegister4Float CA = VectorSubtract(A, C);

		const VectorRegister4Float Normal = VectorNormalize(VectorCross(AB, BC));
		constexpr FRealSingle ThirdFloat = 1.0f / 3.0f;
		constexpr VectorRegister4Float Third = MakeVectorRegisterFloatConstant(ThirdFloat, ThirdFloat, ThirdFloat, ThirdFloat);
		const VectorRegister4Float Centroid = VectorMultiply(VectorAdd(VectorAdd(A, B), C), Third);


		const VectorRegister4Float ABPlane = VectorCross(Normal, AB);
		const VectorRegister4Float ABSigns = VectorBitwiseAnd(ABPlane, SignBit);
		const VectorRegister4Float ABLocalClosest = VectorBitwiseOr(ABSigns, HalfExtents);
		const VectorRegister4Float ABClosestPoint = VectorAdd(ABLocalClosest, Position);

		if (VectorDot3Scalar(VectorSubtract(A, ABClosestPoint), ABPlane) > 0.0)
		{
			return false;
		}

		const VectorRegister4Float BCPlane = VectorCross(Normal, BC);
		const VectorRegister4Float BCSigns = VectorBitwiseAnd(BCPlane, SignBit);
		const VectorRegister4Float BCLocalClosest = VectorBitwiseOr(BCSigns, HalfExtents);
		const VectorRegister4Float BCClosestPoint = VectorAdd(BCLocalClosest, Position);

		if (VectorDot3Scalar(VectorSubtract(B, BCClosestPoint), BCPlane) > 0.0)
		{
			return false;
		}

		const VectorRegister4Float CAPlane = VectorCross(Normal, CA);
		const VectorRegister4Float CASigns = VectorBitwiseAnd(CAPlane, SignBit);
		const VectorRegister4Float CALocalClosest = VectorBitwiseOr(CASigns, HalfExtents);
		const VectorRegister4Float CAClosestPoint = VectorAdd(CALocalClosest, Position);

		if (VectorDot3Scalar(VectorSubtract(C, CAClosestPoint), CAPlane) > 0.0)
		{
			return false;
		}

		// Triangle Plane 
		const VectorRegister4Float Signs = VectorBitwiseAnd(Normal, SignBit);
		const VectorRegister4Float LocalClosest = VectorBitwiseOr(Signs, HalfExtents);
		const VectorRegister4Float ClosestPoint = VectorAdd(LocalClosest, Position);
		const VectorRegister4Float FurthestPoint = VectorSubtract(Position, LocalClosest);

		VectorRegister4Float ClosCent = VectorSubtract(ClosestPoint, Centroid);
		VectorRegister4Float FurtCent = VectorSubtract(FurthestPoint, Centroid);

		FRealSingle ClosestDist = VectorDot3Scalar(Normal, ClosCent);
		FRealSingle FurthestDist = VectorDot3Scalar(Normal, FurtCent);

		if (FMath::Abs<FRealSingle>(ClosestDist) > UE_KINDA_SMALL_NUMBER &&
			FMath::Abs<FRealSingle>(FurthestDist) > UE_KINDA_SMALL_NUMBER &&
			FMath::Sign(ClosestDist) == FMath::Sign(FurthestDist))
		{
			return false;
		}

		if (!ComputeEdgeOverlap(AB, A, Centroid))
		{
			return false;
		}
		if (!ComputeEdgeOverlap(BC, B, Centroid))
		{
			return false;
		}
		return ComputeEdgeOverlap(CA, C, Centroid);
	}
}
