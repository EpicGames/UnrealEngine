// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"

namespace Chaos
{
	template<class T, int d>
	class TBox;

	template<typename TConcrete, bool bInstanced>
	class TImplicitObjectScaled;


	bool ComputeCapsuleTriangleOverlapSimd(const VectorRegister4Float& A, const VectorRegister4Float& B, const VectorRegister4Float& C, const VectorRegister4Float& X1, const VectorRegister4Float& X2, FRealSingle Radius);
	bool ComputeSphereTriangleOverlapSimd(const VectorRegister4Float& A, const VectorRegister4Float& B, const VectorRegister4Float& C, const VectorRegister4Float& X, FRealSingle Radius);

	class FBoxSimd
	{
	public:
		FBoxSimd();
		FBoxSimd(const FRigidTransform3& WorldScaleQueryTM, const TBox<FReal, 3>& InQueryGeom);
		FBoxSimd(const FRigidTransform3& QueryTM, const TImplicitObjectScaled< TBox<FReal, 3>, true >& QueryGeom);

		bool OverlapTriangle(const VectorRegister4Float& A, const VectorRegister4Float& B, const VectorRegister4Float& C) const;

	private:
		void Initialize(const FRigidTransform3& Transform, const FVec3f& HalfExtentsIn);
		bool ComputeEdge(const VectorRegister4Float& PlaneNormal, const VectorRegister4Float& PlaneVertex, const VectorRegister4Float& Edge, const VectorRegister4Float& Centroid) const;
		bool ComputeBoxPlane(const VectorRegister4Float& DistCenter, const VectorRegister4Float& AxisHalfExtent) const;

		VectorRegister4Float Position;
		VectorRegister4Float XAxis;
		VectorRegister4Float YAxis;
		VectorRegister4Float ZAxis;
		VectorRegister4Float XHalfExtent;
		VectorRegister4Float YHalfExtent;
		VectorRegister4Float ZHalfExtent;
		VectorRegister4Float XAxisHalfExtent;
		VectorRegister4Float YAxisHalfExtent;
		VectorRegister4Float ZAxisHalfExtent;
		static constexpr int32 EdgeNum = 3;
		VectorRegister4Float Edges[EdgeNum];
	};

	class FAABBSimd
	{
	public:
		FAABBSimd();
		FAABBSimd(const TBox<FReal, 3>& InQueryGeom);
		FAABBSimd(const TImplicitObjectScaled< TBox<FReal, 3>, true >& QueryGeom);
		FAABBSimd(const FVec3& Translation, const TBox<FReal, 3>& InQueryGeom);
		FAABBSimd(const FVec3& Translation, const TImplicitObjectScaled< TBox<FReal, 3>, true >& QueryGeom);
		bool OverlapTriangle(const VectorRegister4Float& A, const VectorRegister4Float& B, const VectorRegister4Float& C) const;

	private:
		void Initialize(const FVec3& Translation, const FVec3f& HalfExtentsf);
		bool ComputeEdgeOverlap(const VectorRegister4Float& TriangleEdge, const VectorRegister4Float& TriangleVertex, const VectorRegister4Float& Centroid) const;

		VectorRegister4Float Position;
		VectorRegister4Float HalfExtents;
		const static VectorRegister4Float SignBit;
		const static VectorRegister4Float SignX;
		const static VectorRegister4Float SignY;
		const static VectorRegister4Float SignZ;
		static constexpr int32 EdgeNum = 3;
		constexpr static VectorRegister4Float Edges[EdgeNum] = {
			MakeVectorRegisterFloatConstant(1.f, 0.f, 0.f, 0.f),
			MakeVectorRegisterFloatConstant(0.f, 1.f, 0.f, 0.f),
			MakeVectorRegisterFloatConstant(0.f, 0.f, 1.f, 0.f)
		};
	};
}

