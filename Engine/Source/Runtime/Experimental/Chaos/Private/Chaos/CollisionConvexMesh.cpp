// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/CollisionConvexMesh.h"
#include "HAL/IConsoleManager.h"
#include "CompGeom/ConvexHull3.h"

namespace Chaos
{
	// CVars variables for controlling geometry complexity checking and simplification 
	int32 FConvexBuilder::PerformGeometryCheck = 0;

	int32 FConvexBuilder::PerformGeometryReduction = 0;

	int32 FConvexBuilder::VerticesThreshold = 50;

	int32 FConvexBuilder::ComputeHorizonEpsilonFromMeshExtends = 1;

	bool FConvexBuilder::bUseGeometryTConvexHull3 = true;
	bool FConvexBuilder::bUseSimplifierForTConvexHull3 = false;

	FAutoConsoleVariableRef CVarConvexGeometryCheckEnable(TEXT("p.Chaos.ConvexGeometryCheckEnable"), FConvexBuilder::PerformGeometryCheck, TEXT("Perform convex geometry complexity check for Chaos physics."));

	FAutoConsoleVariableRef CVarConvexGeometrySimplifyEnable(TEXT("p.Chaos.PerformGeometryReduction"), FConvexBuilder::PerformGeometryReduction, TEXT("Perform convex geometry simplification to increase performance in Chaos physics."));

	FAutoConsoleVariableRef CVarConvexParticlesWarningThreshold(TEXT("p.Chaos.ConvexParticlesWarningThreshold"), FConvexBuilder::VerticesThreshold, TEXT("Threshold beyond which we warn about collision geometry complexity."));
	
	FAutoConsoleVariableRef CvarUseGeometryTConvexHull3(TEXT("p.Chaos.Convex.UseTConvexHull3Builder"), FConvexBuilder::bUseGeometryTConvexHull3, TEXT("Use the newer Geometry Tools code path for generating convex hulls when default build method is set.[def:true]"));
	FAutoConsoleVariableRef CvarUseSimplifierForGeometryTConvexHull3(TEXT("p.Chaos.Convex.UseSimplifierForTConvexHull3Builder"), FConvexBuilder::bUseSimplifierForTConvexHull3, TEXT("If default build is using the Geometry Tools convex hull algorithm, also use the corresponding simplifier. [def:false]"));


	bool FConvexBuilder::UseConvexHull3(FConvexBuilder::EBuildMethod BuildMethod)
	{
		return (BuildMethod == EBuildMethod::Default && FConvexBuilder::bUseGeometryTConvexHull3)
			|| (BuildMethod == EBuildMethod::ConvexHull3 || BuildMethod == EBuildMethod::ConvexHull3Simplified);
	}

	bool FConvexBuilder::UseConvexHull3Simplifier(EBuildMethod BuildMethod)
	{

		return (BuildMethod == EBuildMethod::Default && FConvexBuilder::bUseGeometryTConvexHull3 && FConvexBuilder::bUseSimplifierForTConvexHull3)
			|| BuildMethod == EBuildMethod::ConvexHull3Simplified;
	}

	void FConvexBuilder::BuildIndices(const TArray<FVec3Type>& InVertices, TArray<int32>& OutResultIndexData, EBuildMethod BuildMethod)
	{
		if (UseConvexHull3(BuildMethod))
		{
			UE::Geometry::TConvexHull3<FRealType> HullCompute;
			const bool bUseSimplifier = UseConvexHull3Simplifier(BuildMethod);
			if (bUseSimplifier)
			{
				HullCompute.bSaveTriangleNeighbors = true;
				HullCompute.SimplificationSettings.DegenerateEdgeTolerance = (FRealType)UE_DOUBLE_SMALL_NUMBER;
			}
			if (HullCompute.Solve<FVec3Type>(InVertices))
			{
				if (bUseSimplifier)
				{
					// Note: currently these tolerances are chosen to be close to the tolerances used by the Convex Builder's calls to MergeFaces
					// Note: A reasonable alternative for DistanceTolerance would be FRealType(1e-3) * OutLocalBounds.Extents().GetMax();
					const FRealType AngleToleranceDeg = (FRealType)1;
					const FRealType DistanceTolerance = (FRealType)1;
					using FPolygonFace = UE::Geometry::TConvexHull3<FRealType>::FPolygonFace;
					TArray<FPolygonFace> Polygons;
					HullCompute.GetSimplifiedFaces(Polygons, [&InVertices](int32 Idx) { return InVertices[Idx]; }, AngleToleranceDeg, DistanceTolerance);
					for (const FPolygonFace& Face : Polygons)
					{
						OutResultIndexData.Reserve(OutResultIndexData.Num() + 3 * (Face.Num() - 2));
						for (int32 Idx = 1; Idx + 1 < Face.Num(); ++Idx)
						{
							OutResultIndexData.Add(Face[0]);
							OutResultIndexData.Add(Face[Idx + 1]);
							OutResultIndexData.Add(Face[Idx]);
						}
					}
				}
				else
				{
					const TArray<UE::Geometry::FIndex3i>& HullTriangles = HullCompute.GetTriangles();
					OutResultIndexData.Reserve(HullTriangles.Num() * 3);
					for (const UE::Geometry::FIndex3i& Tri : HullTriangles)
					{
						// Winding is backwards from what Chaos expects
						OutResultIndexData.Add(Tri[0]);
						OutResultIndexData.Add(Tri[2]);
						OutResultIndexData.Add(Tri[1]);
					}
				}
			}
		}
		else
		{
			TArray<Chaos::TVec3<int32>> Triangles;
			Params BuildParams;
			BuildParams.HorizonEpsilon = SuggestEpsilon(InVertices);
			BuildConvexHull(InVertices, Triangles, BuildParams);
			OutResultIndexData.Reserve(Triangles.Num() * 3);
			for (Chaos::TVec3<int32> Tri : Triangles)
			{
				OutResultIndexData.Add(Tri[0]);
				OutResultIndexData.Add(Tri[1]);
				OutResultIndexData.Add(Tri[2]);
			}
		}

	}

	void FConvexBuilder::Build(const TArray<FVec3Type>& InVertices, TArray <FPlaneType>& OutPlanes, TArray<TArray<int32>>& OutFaceIndices, TArray<FVec3Type>& OutVertices, FAABB3Type& OutLocalBounds, EBuildMethod BuildMethod)
	{
		OutPlanes.Reset();
		OutVertices.Reset();
		OutLocalBounds = FAABB3Type::EmptyAABB();

		const int32 NumVerticesIn = InVertices.Num();
		if (NumVerticesIn == 0)
		{
			return;
		}


		const TArray<FVec3Type>* VerticesToUse = &InVertices;
		TArray<FVec3Type> ModifiedVertices;

		// For triangles and planar shapes, create a very thin prism as a convex
		auto Inflate = [](const TArray<FVec3Type>& Source, TArray<FVec3Type>& Destination, const FVec3Type& Normal, FRealType Inflation)
		{
			const int32 NumSource = Source.Num();
			Destination.Reset();
			Destination.SetNum(NumSource * 2);

			for (int32 Index = 0; Index < NumSource; ++Index)
			{
				Destination[Index] = Source[Index];
				Destination[NumSource + Index] = Source[Index] + Normal * Inflation;
			}
		};

		FVec3Type PlanarNormal(0);
		if (NumVerticesIn == 3)
		{
			const bool bIsValidTriangle = IsValidTriangle(InVertices[0], InVertices[1], InVertices[2], PlanarNormal);

			//TODO_SQ_IMPLEMENTATION: should do proper cleanup to avoid this
			if (ensureMsgf(bIsValidTriangle, TEXT("FConvexBuilder::Build(): Generated invalid triangle!")))
			{
				Inflate(InVertices, ModifiedVertices, PlanarNormal, TriQuadPrismInflation());
				VerticesToUse = &ModifiedVertices;
				UE_LOGF(LogChaos, Verbose, "Encountered a triangle in convex hull generation. Will prepare a prism of thickness %.5f in place of a triangle.", TriQuadPrismInflation());
			}
			else
			{
				return;
			}
		}
		else if (IsPlanarShape(InVertices, PlanarNormal))
		{
			Inflate(InVertices, ModifiedVertices, PlanarNormal, TriQuadPrismInflation());
			VerticesToUse = &ModifiedVertices;
			UE_LOGF(LogChaos, Verbose, "Encountered a planar shape in convex hull generation. Will prepare a prism of thickness %.5f in place of a triangle.", TriQuadPrismInflation());
		}

		const int32 NumVerticesToUse = VerticesToUse->Num();

		OutLocalBounds = FAABB3Type((*VerticesToUse)[0], (*VerticesToUse)[0]);
		for (int32 VertexIndex = 0; VertexIndex < NumVerticesToUse; ++VertexIndex)
		{
			OutLocalBounds.GrowToInclude((*VerticesToUse)[VertexIndex]);
		}

		if (NumVerticesToUse >= 4)
		{
			if (UseConvexHull3(BuildMethod)) // Use the newer Geometry Tools code path for generating convex hulls.
			{
				bool bUseSimplifier = UseConvexHull3Simplifier(BuildMethod);
				UE::Geometry::TConvexHull3<FRealType> HullCompute;
				HullCompute.bSaveTriangleNeighbors = true;
				if (bUseSimplifier)
				{
					HullCompute.SimplificationSettings.DegenerateEdgeTolerance = (FRealType)UE_DOUBLE_SMALL_NUMBER;
				}
				if (HullCompute.Solve<FVec3Type>(*VerticesToUse))
				{
					// Get Planes, FaceIndices, Vertices, and LocalBoundingBox
					TMap<int32, int32> HullVertMap;
					auto AddFace = [&VerticesToUse, &HullVertMap, &OutPlanes, &OutFaceIndices, &OutVertices, &OutLocalBounds](TArray<int32>& FaceIndices, FVector3f FaceNormal)
					{
						TArray<int32>& OutFace = OutFaceIndices.Emplace_GetRef();
						OutFace.SetNumUninitialized(FaceIndices.Num());

						// Note winding is backwards from what Chaos expects, so this fills OutFace in reverse order
						for (int32 j = 0, n = FaceIndices.Num(); j < n; ++j)
						{
							int32 Index = FaceIndices[j];
							int32* MappedVert = HullVertMap.Find(Index);
							if (!MappedVert)
							{
								const FVec3Type& OrigPos = (*VerticesToUse)[Index];
								int32 NewVID = OutVertices.Num();
								OutVertices.Add(OrigPos);
								HullVertMap.Add(Index, NewVID);
								OutFace[n - j - 1] = NewVID;
							}
							else
							{
								OutFace[n - j - 1] = *MappedVert;
							}
						}

						const FVec3Type& Normal = FaceNormal;
						const FVec3Type& Point = OutVertices[OutFace[0]];
						OutPlanes.Add(FPlaneType{ Point, Normal });
					};
					auto GetVertex = [&VerticesToUse](int32 Idx) { return (*VerticesToUse)[Idx]; };
					if (bUseSimplifier)
					{
						// Note: currently these tolerances are chosen to be close to the tolerances used by the Convex Builder's calls to MergeFaces
						// Note: A reasonable alternative for DistanceTolerance would be FRealType(1e-3) * OutLocalBounds.Extents().GetMax();
						const FRealType AngleToleranceDeg = (FRealType)1;
						const FRealType DistanceTolerance = (FRealType)1;
						HullCompute.GetSimplifiedFaces(AddFace, GetVertex, AngleToleranceDeg, DistanceTolerance);
					}
					else
					{
						HullCompute.GetFaces(AddFace, GetVertex);
					}

					OutLocalBounds = FAABB3Type(OutVertices[0], OutVertices[0]);
					for (int32 VertexIndex = 1; VertexIndex < OutVertices.Num(); ++VertexIndex)
					{
						OutLocalBounds.GrowToInclude(OutVertices[VertexIndex]);
					}
				}
			}
			else // Use the older Chaos code path for generating convex hulls.
			{
				// @todo(chaos): Deprecate the older convex code path.
				TArray<TVec3<int32>> Indices;
				Params BuildParams;
				BuildParams.HorizonEpsilon = Chaos::FConvexBuilder::SuggestEpsilon(*VerticesToUse);
				BuildConvexHull(*VerticesToUse, Indices, BuildParams);
				OutPlanes.Reserve(Indices.Num());
				TMap<int32, int32> IndexMap; // maps original particle indices to output particle indices
				int32 NewIdx = 0;

				const auto AddIndex = [&IndexMap, &NewIdx](const int32 OriginalIdx)
				{
					if (int32* Idx = IndexMap.Find(OriginalIdx))
					{
						return *Idx;
					}
					IndexMap.Add(OriginalIdx, NewIdx);
					return NewIdx++;
				};

				for (const TVec3<int32>& Idx : Indices)
				{
					FVec3Type Vs[3] = { (*VerticesToUse)[Idx[0]], (*VerticesToUse)[Idx[1]], (*VerticesToUse)[Idx[2]] };
					const FVec3Type Normal = FVec3Type::CrossProduct(Vs[1] - Vs[0], Vs[2] - Vs[0]).GetUnsafeNormal();
					OutPlanes.Add(FPlaneType(Vs[0], Normal));
					TArray<int32> FaceIndices;
					FaceIndices.SetNum(3);
					FaceIndices[0] = AddIndex(Idx[0]);
					FaceIndices[1] = AddIndex(Idx[1]);
					FaceIndices[2] = AddIndex(Idx[2]);
					OutFaceIndices.Add(FaceIndices);
				}

				OutVertices.SetNum(IndexMap.Num());
				for (const auto& Elem : IndexMap)
				{
					OutVertices[Elem.Value] = (*VerticesToUse)[Elem.Key];
				}
			}
		}

		UE_CLOGF(OutVertices.Num() == 0, LogChaos, Warning, "Convex hull generation produced zero convex particles, collision will fail for this primitive.");
	}

}
