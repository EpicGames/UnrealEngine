// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/CollisionFilterData.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/Collision/CollisionFilter.h"

namespace Chaos::Filter
{
	constexpr uint8 NumFlagsBits = 20;
	constexpr uint8 NumCollisionChannelBits = 6;
	constexpr uint8 NumMaskFilterBits = 6;
	static_assert(NumFlagsBits + NumMaskFilterBits + NumCollisionChannelBits == 32);

	constexpr uint32 CollisionChannelBitsOffset = NumFlagsBits;
	constexpr uint32 MaskFilterBitsOffset = CollisionChannelBitsOffset + NumCollisionChannelBits;

	constexpr uint32 FilterFlagsMask = (1 << NumFlagsBits) - 1;
	constexpr uint32 CollisionChannelMask = ((1 << NumCollisionChannelBits) - 1) << CollisionChannelBitsOffset;
	constexpr uint32 MaskFilterMask = ((1 << NumMaskFilterBits) - 1) << MaskFilterBitsOffset;

	constexpr uint64 FullChannelMask = std::numeric_limits<uint64>::max();

	constexpr uint32 LegacyQueryFlagsMask = (uint32)(EFilterFlags::SimpleCollision | EFilterFlags::ComplexCollision);
	constexpr uint32 LegacySimFlagsMask = (uint32)(EFilterFlags::All) & ~LegacyQueryFlagsMask;

	constexpr uint8 NumLegacyFlagsBits = 21;
	constexpr uint8 NumLegacyCollisionChannelBits = 5;
	static_assert(NumLegacyFlagsBits + NumMaskFilterBits + NumLegacyCollisionChannelBits == 32);
	constexpr uint32 LegacyCollisionChannelBitsOffset = NumLegacyFlagsBits;
	constexpr uint32 LegacyCollisionChannelMask = ((1 << NumLegacyCollisionChannelBits) - 1) << LegacyCollisionChannelBitsOffset;

	uint32 GetFlags(uint32 Word3)
	{
		const uint32 FilterFlags = (Word3 & FilterFlagsMask);
		return FilterFlags;
	}

	void SetFlags(uint32& Word3, uint32 Flags)
	{
		const uint32 Word3Cleared = Word3 & ~FilterFlagsMask;
		Word3 = Word3Cleared | (Flags & FilterFlagsMask);
	}

	void SetFlags(EFilterFlags& CurrentFlags, const EFilterFlags InFilterFlags, bool bEnabled)
	{
		if (bEnabled)
		{
			EnumAddFlags(CurrentFlags, InFilterFlags);
		}
		else
		{
			EnumRemoveFlags(CurrentFlags, InFilterFlags);
		}
	}

	bool HasFlag(uint32 Word3, EFilterFlags InFlag)
	{
		const uint32 FilterFlags = GetFlags(Word3);
		return FilterFlags & static_cast<uint32>(InFlag);
	}

	uint8 GetCollisionChannelIndex(uint32 Word3)
	{
		const uint32 ChannelIndex = (Word3 & CollisionChannelMask) >> CollisionChannelBitsOffset;
		return (uint8)ChannelIndex;
	}

	uint64 GetCollisionChannelMask64(uint32 Word3)
	{
		return (uint64)1 << (uint64)GetCollisionChannelIndex(Word3);
	}

	uint8 GetFirstCollisionChannelIndex(uint64 Mask)
	{
		if (Mask == 0)
		{
			return 0;
		}
		return (uint8)FMath::CountTrailingZeros64(Mask);
	}

	void SetCollisionChannel(uint32& Word3, uint32 Value)
	{
		const uint32 Word3Cleared = Word3 & ~CollisionChannelMask;
		const uint32 NewCollisionChannelBits = CollisionChannelMask & (Value << CollisionChannelBitsOffset);
		Word3 = Word3Cleared | NewCollisionChannelBits;
	}

	uint8 GetMaskFilter(uint32 Word3)
	{
		return Word3 >> (32u - NumMaskFilterBits);
	}

	void SetMaskFilter(uint32& Word3, uint8 InMaskFilter)
	{
		// Mask the input to make sure there's no bit overflows
		constexpr uint32 InFilterMask = (1 << NumMaskFilterBits) - 1;
		const uint32 MaskFilter = uint32(InMaskFilter) & InFilterMask;

		static_assert(NumMaskFilterBits <= 8, "Only up to 8 extra filter bits are supported.");
		// Clear the old mask filter bits
		Word3 &= ~MaskFilterMask;
		// Set the new mask filter bits
		Word3 |= MaskFilter << MaskFilterBitsOffset;
	}

	uint32 LegacyToNewWord3(const uint32 LegacyWord3)
	{
		// Split the word into the collision channel and everything else.
		const uint32 LegacyChannels = LegacyWord3 & LegacyCollisionChannelMask;
		const uint32 LegacyWord3Remainder = LegacyWord3 & ~LegacyCollisionChannelMask;
		// To build the new word 3, we have to change from 5 bits to 6 bits for the channel index (left aligned).
		// Simply right shift and steal the old upper flags bit (unused).
		const uint32 NewChannels = (LegacyChannels >> 1);
		const uint32 NewWord3 = LegacyWord3Remainder | NewChannels;
		return NewWord3;
	}

	uint32 NewToLegacyWord3(const uint32 NewWord3)
	{
		// Split the word into the collision channel and everything else.
		const uint32 NewChannels = NewWord3 & CollisionChannelMask;
		const uint32 NewWord3Remainder = NewWord3 & ~CollisionChannelMask;
		// To build the legacy word3, we have to change from 6 bits to 5 bits for the channel index (left aligned). 
		// To do this, shift 1 to the left and mask the word (chopping off the top bit so it doesn't interfere with the mask filter).
		const uint32 LegacyChannels = (NewChannels << 1) & LegacyCollisionChannelMask;
		const uint32 LegacyWord3 = NewWord3Remainder | LegacyChannels;
		return LegacyWord3;
	}

	FInstanceData::FInstanceData(const uint32 OwnerId, const uint32 ComponentId)
		: OwnerId(OwnerId)
		, ComponentId(ComponentId)
	{
	}

	bool FInstanceData::IsValid() const
	{
		return OwnerId != 0 || ComponentId != 0;
	}

	uint32 FInstanceData::GetActorId() const
	{
		return GetOwnerId();
	}

	void FInstanceData::SetActorId(const uint32 InActorId)
	{
		SetOwnerId(InActorId);
	}

	uint32 FInstanceData::GetOwnerId() const
	{
		return OwnerId;
	}

	void FInstanceData::SetOwnerId(const uint32 InOwnerId)
	{
		OwnerId = InOwnerId;
	}

	uint32 FInstanceData::GetComponentId() const
	{
		return ComponentId;
	}

	void FInstanceData::SetComponentId(const uint32 InComponentId)
	{
		ComponentId = InComponentId;
	}

	void FInstanceData::Serialize(FChaosArchive& Ar)
	{
		Ar << OwnerId;
		Ar << ComponentId;
	}

	FString FInstanceData::ToString() const
	{
		return FString::Format(TEXT("OwnerId({0}) ComponentId({1})"), { GetOwnerId(), GetComponentId() });
	}

	bool FShapeFilterData::IsValid() const
	{
		return BlockChannels != 0 || OverlapChannels != 0 || Word3 != 0;
	}

	bool FShapeFilterData::IsSimValid() const
	{
		return BlockChannels != 0 || Word3 != 0;
	}

	EFilterFlags FShapeFilterData::GetFlags() const
	{
		return (EFilterFlags)Chaos::Filter::GetFlags(Word3);
	}

	void FShapeFilterData::SetFlags(EFilterFlags InFlags)
	{
		Chaos::Filter::SetFlags(Word3, (uint32)InFlags);
	}

	bool FShapeFilterData::HasFlag(EFilterFlags InFlag) const
	{
		return Chaos::Filter::HasFlag(Word3, InFlag);
	}

	uint8 FShapeFilterData::GetMaskFilter() const
	{
		return Chaos::Filter::GetMaskFilter(Word3);
	}

	void FShapeFilterData::SetMaskFilter(uint8 MaskFilter)
	{
		Chaos::Filter::SetMaskFilter(Word3, MaskFilter);
	}

	uint8 FShapeFilterData::GetCollisionChannelIndex() const
	{
		return Chaos::Filter::GetCollisionChannelIndex(Word3);
	}

	uint64 FShapeFilterData::GetCollisionChannelMask() const
	{
		return Chaos::Filter::GetCollisionChannelMask64(Word3);
	}

	bool FShapeFilterData::IsCollisionChannelSet(const uint32 ChannelIndex) const
	{
		if (ChannelIndex < 64)
		{
			const uint64 Mask = 1LLU << ChannelIndex;
			return GetCollisionChannelMask() & Mask;
		}
		return false;
	}

	uint64 FShapeFilterData::GetBlockChannels() const
	{
		return BlockChannels;
	}

	uint64 FShapeFilterData::GetOverlapChannels() const
	{
		return OverlapChannels;
	}

	uint64 FShapeFilterData::GetQueryBlockChannels() const
	{
		return BlockChannels;
	}

	uint64 FShapeFilterData::GetQueryOverlapChannels() const
	{
		return OverlapChannels;
	}

	uint64 FShapeFilterData::GetSimBlockChannels() const
	{
		return BlockChannels;
	}

	ENarrowFilterResult FShapeFilterData::NarrowFilter(const FShapeFilterData& OtherShape) const
	{
		const uint64 ChannelMask0 = GetCollisionChannelMask();
		const uint64 ChannelMask1 = OtherShape.GetCollisionChannelMask();
		const uint64 BlockMask0 = GetBlockChannels();
		const uint64 BlockMask1 = OtherShape.GetBlockChannels();

		const bool bBlocking = (ChannelMask0 & BlockMask1) && (ChannelMask1 & BlockMask0);
		return bBlocking ? ENarrowFilterResult::Block : ENarrowFilterResult::None;
	}

	void FShapeFilterData::Serialize(FChaosArchive& Ar)
	{
		Ar << BlockChannels;
		Ar << OverlapChannels;

		// Pre-emptively serialize the channel as a separate mask. This will allow easy extending to multiple channels later
		if (Ar.IsLoading())
		{
			uint64 ChannelMask;
			Ar << ChannelMask;
			Ar << Word3;
			uint8 ChannelIndex = GetFirstCollisionChannelIndex(ChannelMask);
			SetCollisionChannel(Word3, ChannelIndex);
		}
		else
		{
			uint32 TempWord3 = Word3;
			TempWord3 &= ~CollisionChannelMask;
			uint64 ChannelMask = GetCollisionChannelMask64(Word3);
			Ar << ChannelMask;
			Ar << TempWord3;
		}
	}

	FString FShapeFilterData::ToString() const
	{
		return FString::Printf(TEXT("Block(0x%llX) Overlap(0x%llX) CollisionChannel(0x%X) MaskFilter(0x%X) Flags(0x%X)"), GetBlockChannels(), GetOverlapChannels(), GetCollisionChannelIndex(), GetMaskFilter(), (uint8)GetFlags());
	}

	uint32 FShapeFilterData::GetLegacyWord3() const
	{
		return NewToLegacyWord3(Word3);
	}

	void FShapeFilterData::Store(FCollisionFilterData& QueryData, FCollisionFilterData& SimData) const
	{
		QueryData.Word2 = uint32(BlockChannels >> 32);
		QueryData.Word3 = uint32(BlockChannels) & 0xFFFFFFFF;
		SimData.Word0 = uint32(OverlapChannels >> 32);
		SimData.Word1 = uint32(OverlapChannels) & 0xFFFFFFFF;
		SimData.Word2 = Word3;
	}

	void FShapeFilterData::Load(const FCollisionFilterData& QueryData, const FCollisionFilterData& SimData)
	{
		BlockChannels = (uint64(QueryData.Word2) << 32) | uint64(QueryData.Word3);
		OverlapChannels = (uint64(SimData.Word0) << 32) | uint64(SimData.Word1);
		Word3 = SimData.Word2;
	}

	void FShapeUnionFilterData::Combine(const FShapeFilterData& ShapeFilter, bool bQueryEnabled, bool bSimEnabled)
	{
		if (bQueryEnabled)
		{
			// Note: Query Channels are unused in broad filtering. 
			// Including them would increase the size of this filter which would increase the acceleration handle size.
			QueryBlockChannels |= ShapeFilter.BlockChannels;
			QueryOverlapChannels |= ShapeFilter.OverlapChannels;
		}
		if (bSimEnabled)
		{
			SimBlockChannels |= ShapeFilter.BlockChannels;
			SimChannelMask |= ShapeFilter.GetCollisionChannelMask();
		}
	}

	EBroadFilterResult FShapeUnionFilterData::BroadFilter(const FShapeUnionFilterData& OtherFilter) const
	{
		const uint64 ChannelMask0 = SimChannelMask;
		const uint64 ChannelMask1 = OtherFilter.SimChannelMask;
		const uint64 BlockMask0 = SimBlockChannels;
		const uint64 BlockMask1 = OtherFilter.SimBlockChannels;

		const bool bCanCollide = (ChannelMask0 & BlockMask1) && (ChannelMask1 & BlockMask0);
		return bCanCollide ? EBroadFilterResult::Accept : EBroadFilterResult::Reject;
	}

	EBroadFilterResult FShapeUnionFilterData::BroadFilter(const FQueryFilterData& QueryFilter) const
	{
		if (QueryFilter.GetQueryType() == FQueryFilterData::EQueryType::Channel)
		{
			const uint64 QueryFilterChannelMask = QueryFilter.GetCollisionChannelMask();
			const uint64 CombinedMask = QueryBlockChannels | QueryOverlapChannels;
			const bool bIsHit = QueryFilterChannelMask & CombinedMask;
			return bIsHit ? EBroadFilterResult::Accept : EBroadFilterResult::Reject;
		}
		return EBroadFilterResult::Accept;
	}

	bool FQueryFilterData::IsValid() const
	{
		return Word0 != 0 || Word1 != 0 || Word2 != 0 || Word3 != 0;
	}

	FQueryFilterData::EQueryType FQueryFilterData::GetQueryType() const
	{
		return (Word0 & 0x1) != 0 ? EQueryType::Channel : EQueryType::ObjectType;
	}

	EFilterFlags FQueryFilterData::GetFlags() const
	{
		return (EFilterFlags)Chaos::Filter::GetFlags(Word3);
	}

	void FQueryFilterData::SetFlags(EFilterFlags InFlags)
	{
		Chaos::Filter::SetFlags(Word3, (uint32)InFlags);
	}

	bool FQueryFilterData::HasFlag(EFilterFlags InFlag) const
	{
		return Chaos::Filter::HasFlag(Word3, InFlag);
	}

	uint8 FQueryFilterData::GetIgnoreMask() const
	{
		return GetMaskFilter(Word3);
	}

	uint8 FQueryFilterData::GetCollisionChannelIndex() const
	{
		return Chaos::Filter::GetCollisionChannelIndex(Word3);
	}

	uint64 FQueryFilterData::GetCollisionChannelMask() const
	{
		return Chaos::Filter::GetCollisionChannelMask64(Word3);
	}

	bool FQueryFilterData::IsCollisionChannelSet(const uint32 ChannelIndex) const
	{
		if (ChannelIndex < 64)
		{
			const uint64 Mask = 1LLU << ChannelIndex;
			return GetCollisionChannelMask() & Mask;
		}
		return false;
	}

	uint64 FQueryFilterData::GetBlockChannels() const
	{
		return Word1;
	}

	uint64 FQueryFilterData::GetOverlapChannels() const
	{
		return Word2;
	}

	uint64 FQueryFilterData::GetObjectTypesToQueryMask() const
	{
		return Word1;
	}

	bool FQueryFilterData::IsMultiQuery() const
	{
		const uint32 MultiTrace = GetCollisionChannelIndex();
		return MultiTrace != 0;
	}

	ENarrowFilterResult FQueryFilterData::NarrowFilter(const FShapeFilterData& ShapeFilter) const
	{
		const FMaskFilter QuerierMaskFilter = GetIgnoreMask();
		const FMaskFilter ShapeMaskFilter = ShapeFilter.GetMaskFilter();

		// Check if the ignore masks say to skip
		if ((QuerierMaskFilter & ShapeMaskFilter) != 0)
		{
			return ENarrowFilterResult::None;
		}

		if (GetQueryType() == FQueryFilterData::EQueryType::ObjectType)
		{
			return ObjectTypeNarrowFilter(ShapeFilter);
		}
		else
		{
			return ChannelTypeNarrowFilter(ShapeFilter);
		}
	}

	CHAOS_API FString FQueryFilterData::ToString() const
	{
		if (GetQueryType() == EQueryType::Channel)
		{
			return FString::Printf(TEXT("Type(Channel) Block(0x%llX) Overlap(0x%llX) ChannelIndex(0x%X) IgnoreMask(0x%X) Flags(0x%X)"), GetBlockChannels(), GetOverlapChannels(), GetCollisionChannelIndex(), GetIgnoreMask(), (uint8)GetFlags());
		}
		else
		{
			return FString::Printf(TEXT("Type(Object) IsMulti(%d) ObjectTypes(0x%llX) IgnoreMask(0x%x) Flags(0x%X)"), IsMultiQuery(), GetObjectTypesToQueryMask(), GetIgnoreMask(), (uint8)GetFlags());
		}
	}

	ENarrowFilterResult FQueryFilterData::ChannelTypeNarrowFilter(const FShapeFilterData& ShapeFilter) const
	{
		const uint64 QueryChannelMask = GetCollisionChannelMask();
		const uint64 QueryBlockChannels = GetBlockChannels();
		const uint64 QueryOverlapChannels = GetOverlapChannels();

		const uint64 ShapeChannelMask = ShapeFilter.GetCollisionChannelMask();
		const uint64 ShapeBlockChannels = ShapeFilter.GetBlockChannels();
		const uint64 ShapeOverlapChannels = ShapeFilter.GetOverlapChannels();

		ENarrowFilterResult QuerierHitType = ENarrowFilterResult::None;
		ENarrowFilterResult ShapeHitType = ENarrowFilterResult::None;

		// Check query type vs. shape masks
		if ((QueryChannelMask & ShapeBlockChannels) != 0)
		{
			QuerierHitType = ENarrowFilterResult::Block;
		}
		else if ((QueryChannelMask & ShapeOverlapChannels) != 0)
		{
			QuerierHitType = ENarrowFilterResult::Overlap;
		}

		// Check shape type vs. query masks
		if ((ShapeChannelMask & QueryBlockChannels) != 0)
		{
			ShapeHitType = ENarrowFilterResult::Block;
		}
		else if ((ShapeChannelMask & QueryOverlapChannels) != 0)
		{
			ShapeHitType = ENarrowFilterResult::Overlap;
		}

		// Combine the results by taking the min
		return FMath::Min(QuerierHitType, ShapeHitType);
	}

	ENarrowFilterResult FQueryFilterData::ObjectTypeNarrowFilter(const FShapeFilterData& ShapeFilter) const
	{
		const uint64 QueryObjectTypesMask = GetObjectTypesToQueryMask();
		const uint64 ShapeChannelMask = ShapeFilter.GetCollisionChannelMask();
		// do I belong to one of objects of interest?
		if (ShapeChannelMask & QueryObjectTypesMask)
		{
			return ENarrowFilterResult::Block;
		}
		return ENarrowFilterResult::None;
	}

	FCombinedShapeFilterData::FCombinedShapeFilterData(const FShapeFilterData& InShapeFilter, const FInstanceData& InInstanceData)
		: ShapeFilterData(InShapeFilter)
		, InstanceData(InInstanceData)
	{
	}

	const FInstanceData& FCombinedShapeFilterData::GetInstanceData() const
	{
		return InstanceData;
	}

	void FCombinedShapeFilterData::SetInstanceData(const FInstanceData& InData)
	{
		InstanceData = InData;
	}

	const FShapeFilterData& FCombinedShapeFilterData::GetShapeFilterData() const
	{
		return ShapeFilterData;
	}

	void FCombinedShapeFilterData::SetShapeFilterData(const FShapeFilterData& InData)
	{
		ShapeFilterData = InData;
	}

	bool FCombinedShapeFilterData::IsValid() const
	{
		return ShapeFilterData.IsValid() || InstanceData.IsValid();
	}

	bool FCombinedShapeFilterData::IsSimValid() const
	{
		FCollisionFilterData SimFilter = FShapeFilterBuilder::GetLegacyShapeSimFilter(*this);
		return SimFilter.Word0 != 0 || SimFilter.Word1 != 0 || SimFilter.Word2 != 0 || SimFilter.Word3 != 0;
	}

	FShapeFilterBuilder::FShapeFilterBuilder(const FShapeFilterData& InFilterData)
		: ShapeFilterData(InFilterData)
	{
	}

	FShapeFilterBuilder& FShapeFilterBuilder::SetCollisionChannelIndex(const uint8 ChannelIndex)
	{
		SetCollisionChannel(ShapeFilterData.Word3, (uint32)ChannelIndex);
		return *this;
	}

	FShapeFilterBuilder& FShapeFilterBuilder::SetBlockChannelMask(const uint64 BlockMask)
	{
		ShapeFilterData.BlockChannels = BlockMask;
		return *this;
	}

	FShapeFilterBuilder& FShapeFilterBuilder::SetOverlapChannelMask(const uint64 OverlapMask)
	{
		ShapeFilterData.OverlapChannels = OverlapMask;
		return *this;
	}

	FShapeFilterBuilder& FShapeFilterBuilder::SetMaskFilter(const uint8 InMaskFilter = 0)
	{
		ShapeFilterData.SetMaskFilter(InMaskFilter);
		return *this;
	}

	FShapeFilterBuilder& FShapeFilterBuilder::SetFilterFlags(const EFilterFlags InFilterFlags)
	{
		ShapeFilterData.SetFlags(InFilterFlags);
		return *this;
	}

	FShapeFilterBuilder& FShapeFilterBuilder::SetFilterFlags(const EFilterFlags InFilterFlags, bool bEnabled)
	{
		EFilterFlags NewFlags = ShapeFilterData.GetFlags();
		SetFlags(NewFlags, InFilterFlags, bEnabled);
		return SetFilterFlags(NewFlags);
	}

	FShapeFilterData FShapeFilterBuilder::Build() const
	{
		return ShapeFilterData;
	}

	FShapeFilterData FShapeFilterBuilder::BuildBlockAll(const EFilterFlags InFilterFlags)
	{
		FShapeFilterBuilder Builder;
		Builder.SetBlockChannelMask(FullChannelMask);
		Builder.SetFilterFlags(InFilterFlags);
		return Builder.Build();
	}

	FShapeFilterData FShapeFilterBuilder::BuildFromLegacySimFilter(const FCollisionFilterData& SimFilter)
	{
		FShapeFilterData Result;
		Result.BlockChannels = SimFilter.Word1;
		Result.Word3 = LegacyToNewWord3(SimFilter.Word3);
		return Result;
	}

	FCombinedShapeFilterData FShapeFilterBuilder::BuildFromLegacyShapeFilter(const FCollisionFilterData& QueryFilterData, const FCollisionFilterData& SimFilterData)
	{
		// This is often called with logic like: (bComplex ? SimpleFilter : ComplexFilter, SimFilter).
		// In this case, the sim filter won't match because of the simple/complex flags.
		// Word3 should be separate apart from these flags though, so simply pretend we're setting a new query filter.
		// Do a basic ensure that the filter mask and collision channels are the same, if not this is a bug.
		if (QueryFilterData.Word3 != 0 && SimFilterData.Word3 != 0)
		{
			const uint32 SimFilterMask = SimFilterData.Word3 & MaskFilterMask;
			const uint32 SimCollisionChannel = SimFilterData.Word3 & LegacyCollisionChannelMask;
			const uint32 QueryFilterMask = QueryFilterData.Word3 & MaskFilterMask;
			const uint32 QueryCollisionChannel = QueryFilterData.Word3 & LegacyCollisionChannelMask;
			ensure(SimFilterMask == QueryFilterMask);
			ensure(SimCollisionChannel == QueryCollisionChannel);
		}
		const uint32 Word3 = GetWord3FromNewQueryFilter(QueryFilterData.Word3, SimFilterData.Word3);


		FCombinedShapeFilterData CombinedShapeFilterData;
		CombinedShapeFilterData.InstanceData.OwnerId = QueryFilterData.Word0;
		CombinedShapeFilterData.InstanceData.ComponentId = SimFilterData.Word2;

		ensureMsgf(QueryFilterData.Word1 == SimFilterData.Word1, TEXT("Legacy filters had differing block channels. Query(0x%X) Sim(0x%x). Using query block channels."), QueryFilterData.Word1, SimFilterData.Word1);
		CombinedShapeFilterData.ShapeFilterData.BlockChannels = QueryFilterData.Word1;
		CombinedShapeFilterData.ShapeFilterData.OverlapChannels = QueryFilterData.Word2;
		CombinedShapeFilterData.ShapeFilterData.Word3 = LegacyToNewWord3(Word3);

		return CombinedShapeFilterData;
	}

	void FShapeFilterBuilder::SetLegacyShapeQueryFilter(FCombinedShapeFilterData& CombinedShapeFilterData, const FCollisionFilterData& QueryFilterData)
	{
		FCollisionFilterData SimFilterData = GetLegacyShapeSimFilter(CombinedShapeFilterData);
		const uint32 BlockChannels = QueryFilterData.Word1;
		const uint32 Word3 = GetWord3FromNewQueryFilter(QueryFilterData.Word3, SimFilterData.Word3);
		BuildFromLegacyShapeFilter(QueryFilterData, SimFilterData, BlockChannels, Word3, CombinedShapeFilterData);
	}

	void FShapeFilterBuilder::SetLegacyShapeSimFilter(FCombinedShapeFilterData& CombinedShapeFilterData, const FCollisionFilterData& SimFilterData)
	{
		const FCollisionFilterData QueryFilterData = GetLegacyShapeQueryFilter(CombinedShapeFilterData);
		const uint32 BlockChannels = SimFilterData.Word1;
		const uint32 Word3 = GetWord3FromNewSimFilter(SimFilterData.Word3, QueryFilterData.Word3);
		BuildFromLegacyShapeFilter(QueryFilterData, SimFilterData, BlockChannels, Word3, CombinedShapeFilterData);
	}

	void FShapeFilterBuilder::GetLegacyShapeFilter(const FCombinedShapeFilterData& CombinedShapeFilterData, FCollisionFilterData& OutQueryFilterData, FCollisionFilterData& OutSimFilterData)
	{
		const FShapeFilterData& ShapeFilterData = CombinedShapeFilterData.ShapeFilterData;
		const uint32 Word3 = NewToLegacyWord3(ShapeFilterData.Word3);

		OutQueryFilterData.Word0 = CombinedShapeFilterData.InstanceData.OwnerId;
		OutSimFilterData.Word2 = CombinedShapeFilterData.InstanceData.ComponentId;

		OutQueryFilterData.Word1 = (uint32)CombinedShapeFilterData.ShapeFilterData.BlockChannels;
		OutQueryFilterData.Word2 = (uint32)CombinedShapeFilterData.ShapeFilterData.OverlapChannels;
		OutQueryFilterData.Word3 = Word3;
		OutSimFilterData.Word1 = (uint32)CombinedShapeFilterData.ShapeFilterData.BlockChannels;
		OutSimFilterData.Word3 = Word3;
	}

	FCollisionFilterData FShapeFilterBuilder::GetLegacyShapeQueryFilter(const FCombinedShapeFilterData& CombinedShapeFilterData)
	{
		FCollisionFilterData QueryFilter, SimFilter;
		GetLegacyShapeFilter(CombinedShapeFilterData, QueryFilter, SimFilter);
		return QueryFilter;
	}

	FCollisionFilterData FShapeFilterBuilder::GetLegacyShapeSimFilter(const FCombinedShapeFilterData& CombinedShapeFilterData)
	{
		FCollisionFilterData QueryFilter, SimFilter;
		GetLegacyShapeFilter(CombinedShapeFilterData, QueryFilter, SimFilter);
		return SimFilter;
	}

	uint32 FShapeFilterBuilder::GetWord3FromNewSimFilter(const uint32 SimWord3, const uint32 QueryWord3)
	{
		// If the sim's word3 isn't set, this is likely legacy setting an empty sim word or an ordering issue, so we just use the query word3.
		// Otherwise, the new word3 is the sim's word3 with a few bits stolen from the flag of the query word3 (simple/complex only affect queries currently).
		if (SimWord3 != 0)
		{
			const uint32 MaskFilter = SimWord3 & MaskFilterMask;
			const uint32 ChannelIndex = SimWord3 & LegacyCollisionChannelMask;
			const uint32 Flags = (SimWord3 & LegacySimFlagsMask) | (QueryWord3 & LegacyQueryFlagsMask);
			return MaskFilter | ChannelIndex | Flags;
		}
		return QueryWord3;
	}

	uint32 FShapeFilterBuilder::GetWord3FromNewQueryFilter(const uint32 QueryWord3, const uint32 SimWord3)
	{
		if (QueryWord3 != 0)
		{
			const uint32 MaskFilter = QueryWord3 & MaskFilterMask;
			const uint32 ChannelIndex = QueryWord3 & LegacyCollisionChannelMask;
			const uint32 Flags = (SimWord3 & LegacySimFlagsMask) | (QueryWord3 & LegacyQueryFlagsMask);
			return MaskFilter | ChannelIndex | Flags;
		}
		return SimWord3;
	}

	FCombinedShapeFilterData FShapeFilterBuilder::LoadFromLegacySerialization(const FCollisionFilterData& QueryFilterData, const FCollisionFilterData& SimFilterData)
	{
		uint32 Word3 = QueryFilterData.Word3;
		if (QueryFilterData.Word3 != SimFilterData.Word3)
		{
			const uint32 QueryMaskFilter = QueryFilterData.Word3 & MaskFilterMask;
			const uint32 QueryChannelIndex = QueryFilterData.Word3 & LegacyCollisionChannelMask;
			const uint32 QueryFlags = QueryFilterData.Word3 & LegacyQueryFlagsMask;
			const uint32 SimMaskFilter = SimFilterData.Word3 & MaskFilterMask;
			const uint32 SimChannelIndex = SimFilterData.Word3 & LegacyCollisionChannelMask;
			const uint32 SimFlags = SimFilterData.Word3 & LegacySimFlagsMask;
			UE_CLOGF(QueryMaskFilter != SimMaskFilter, LogChaos, Warning, "Deserializing collision filters with different mask filters: Query(0x%X), Sim(0x%X). Query mask filter will be used.", QueryMaskFilter >> MaskFilterBitsOffset, SimMaskFilter >> MaskFilterBitsOffset);
			UE_CLOGF(QueryChannelIndex != SimChannelIndex, LogChaos, Warning, "Deserializing collision filters with different collision channels: Query(%d), Sim(%d). Query collision channel will be used.", QueryChannelIndex >> LegacyCollisionChannelBitsOffset, SimChannelIndex >> LegacyCollisionChannelBitsOffset);

			const uint32 MaskFilter = QueryMaskFilter != 0 ? QueryMaskFilter : SimMaskFilter;
			const uint32 ChannelIndex = QueryChannelIndex != 0 ? QueryChannelIndex : SimChannelIndex;
			const uint32 Flags = QueryFlags | SimFlags;
			Word3 = MaskFilter | ChannelIndex | Flags;
		}
		const uint32 QueryBlockMask = QueryFilterData.Word1;
		const uint32 SimBlockMask = SimFilterData.Word1;
		uint32 BlockMask = QueryBlockMask;
		UE_CLOGF(QueryBlockMask != SimBlockMask, LogChaos, Warning, "Deserializing collision filters with different block masks: Query(0x%X), Sim(0x%X). Query block mask will be used.", QueryBlockMask, SimBlockMask);

		FCombinedShapeFilterData CombinedShapeFilterData;
		CombinedShapeFilterData.InstanceData.OwnerId = QueryFilterData.Word0;
		CombinedShapeFilterData.InstanceData.ComponentId = SimFilterData.Word2;

		CombinedShapeFilterData.ShapeFilterData.BlockChannels = BlockMask;
		CombinedShapeFilterData.ShapeFilterData.OverlapChannels = QueryFilterData.Word2;
		CombinedShapeFilterData.ShapeFilterData.Word3 = LegacyToNewWord3(Word3);

		return CombinedShapeFilterData;
	}

	void FShapeFilterBuilder::BuildFromLegacyShapeFilter(const FCollisionFilterData& QueryFilterData, const FCollisionFilterData& SimFilterData, const uint32 BlockChannels, const uint32 Word3, FCombinedShapeFilterData& OutCombinedShapeFilterData)
	{
		OutCombinedShapeFilterData.InstanceData.OwnerId = QueryFilterData.Word0;
		OutCombinedShapeFilterData.InstanceData.ComponentId = SimFilterData.Word2;

		OutCombinedShapeFilterData.ShapeFilterData.BlockChannels = BlockChannels;
		OutCombinedShapeFilterData.ShapeFilterData.OverlapChannels = QueryFilterData.Word2;
		OutCombinedShapeFilterData.ShapeFilterData.Word3 = LegacyToNewWord3(Word3);
	}

	FQueryFilterData FQueryFilterBuilder::BuildFromLegacyQueryFilter(const FCollisionFilterData& QueryFilterData)
	{
		FQueryFilterData Result;
		Result.Word0 = QueryFilterData.Word0;
		Result.Word1 = QueryFilterData.Word1;
		Result.Word2 = QueryFilterData.Word2;
		Result.Word3 = LegacyToNewWord3(QueryFilterData.Word3);
		return Result;
	}

	FCollisionFilterData FQueryFilterBuilder::GetLegacyQueryFilter(const FQueryFilterData& QueryFilterData)
	{
		FCollisionFilterData Result;
		Result.Word0 = QueryFilterData.Word0;
		Result.Word1 = (uint32)QueryFilterData.Word1;
		Result.Word2 = (uint32)QueryFilterData.Word2;
		Result.Word3 = NewToLegacyWord3(QueryFilterData.Word3);
		return Result;
	}

	FQueryObjectFilterBuilder::FQueryObjectFilterBuilder()
	{
		QueryFilterData.Word0 = 0;
	}

	FQueryObjectFilterBuilder::FQueryObjectFilterBuilder(const FQueryFilterData& InFilter)
		: QueryFilterData(InFilter)
	{
		QueryFilterData.Word0 = 0;
	}

	FQueryObjectFilterBuilder& FQueryObjectFilterBuilder::SetObjectTypes(uint64 ObjectTypes)
	{
		QueryFilterData.Word1 = ObjectTypes;
		return *this;
	}

	FQueryObjectFilterBuilder& FQueryObjectFilterBuilder::SetMultiQuery(bool bMultiQuery)
	{
		const uint32 MultiQueryValue = bMultiQuery ? 0x1 : 0x0;
		SetCollisionChannel(QueryFilterData.Word3, MultiQueryValue);
		return *this;
	}

	FQueryObjectFilterBuilder& FQueryObjectFilterBuilder::SetMaskFilter(const uint8 InMaskFilter)
	{
		Filter::SetMaskFilter(QueryFilterData.Word3, InMaskFilter);
		return *this;
	}

	FQueryObjectFilterBuilder& FQueryObjectFilterBuilder::SetFilterFlags(const EFilterFlags InFilterFlags)
	{
		QueryFilterData.SetFlags(InFilterFlags);
		return *this;
	}

	FQueryObjectFilterBuilder& FQueryObjectFilterBuilder::SetFilterFlags(const EFilterFlags InFilterFlags, bool bEnabled)
	{
		EFilterFlags NewFlags = QueryFilterData.GetFlags();
		SetFlags(NewFlags, InFilterFlags, bEnabled);
		return SetFilterFlags(NewFlags);
	}

	FQueryFilterData FQueryObjectFilterBuilder::Build() const
	{
		return QueryFilterData;
	}

	FQueryFilterData FQueryObjectFilterBuilder::BuildBlockAll()
	{
		FQueryObjectFilterBuilder Builder;
		Builder.SetObjectTypes(FullChannelMask);
		return Builder.Build();
	}

	FQueryTraceFilterBuilder::FQueryTraceFilterBuilder()
	{
		QueryFilterData.Word0 = 1;
	}

	FQueryTraceFilterBuilder::FQueryTraceFilterBuilder(const FQueryFilterData& InFilter)
		: QueryFilterData(InFilter)
	{
		QueryFilterData.Word0 = 1;
	}

	FQueryTraceFilterBuilder& FQueryTraceFilterBuilder::SetCollisionChannelIndex(const uint8 CollisionChannelIndex)
	{
		SetCollisionChannel(QueryFilterData.Word3, (uint32)CollisionChannelIndex);
		return *this;
	}

	FQueryTraceFilterBuilder& FQueryTraceFilterBuilder::SetBlockChannelMask(const uint64 BlockChannelMask)
	{
		QueryFilterData.Word1 = BlockChannelMask;
		return *this;
	}

	FQueryTraceFilterBuilder& FQueryTraceFilterBuilder::SetOverlapChannelMask(const uint64 OverlapChannelMask)
	{
		QueryFilterData.Word2 = OverlapChannelMask;
		return *this;
	}

	FQueryTraceFilterBuilder& FQueryTraceFilterBuilder::SetMaskFilter(const uint8 InMaskFilter)
	{
		Filter::SetMaskFilter(QueryFilterData.Word3, InMaskFilter);
		return *this;
	}

	FQueryTraceFilterBuilder& FQueryTraceFilterBuilder::SetFilterFlags(const EFilterFlags InFilterFlags)
	{
		QueryFilterData.SetFlags(InFilterFlags);
		return *this;
	}

	FQueryTraceFilterBuilder& FQueryTraceFilterBuilder::SetFilterFlags(const EFilterFlags InFilterFlags, bool bEnabled)
	{
		EFilterFlags NewFlags = QueryFilterData.GetFlags();
		SetFlags(NewFlags, InFilterFlags, bEnabled);
		return SetFilterFlags(NewFlags);
	}

	FQueryFilterData FQueryTraceFilterBuilder::Build() const
	{
		return QueryFilterData;
	}

	FQueryFilterData FQueryTraceFilterBuilder::BuildOverlapAll(const EFilterFlags InFilterFlags)
	{
		Chaos::Filter::FQueryTraceFilterBuilder Builder;
		Builder.SetOverlapChannelMask(FullChannelMask);
		Builder.SetFilterFlags(InFilterFlags);
		return Builder.Build();
	}
} // namespace Chaos::Filter
