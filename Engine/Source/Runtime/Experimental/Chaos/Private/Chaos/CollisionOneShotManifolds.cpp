// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/CollisionOneShotManifolds.h"

#include "Chaos/Box.h"
#include "Chaos/CollisionResolution.h"
#include "Chaos/Collision/PBDCollisionConstraint.h"
#include "Chaos/Convex.h"
#include "Chaos/Defines.h"
#include "Chaos/GJK.h"
#include "Chaos/ImplicitObjectScaled.h"
#include "Chaos/Transform.h"
#include "Chaos/Triangle.h"
#include "Chaos/Utilities.h"
#include "ChaosStats.h"

#include "HAL/IConsoleManager.h"

//PRAGMA_DISABLE_OPTIMIZATION

namespace Chaos
{
	// If a sphere is this much smaller than a capsule, we create a single-point manifold. Otherwise a three-point manifold
	FRealSingle Chaos_Collision_Manifold_SphereCapsuleSizeThreshold = 0.5f;
	FAutoConsoleVariableRef CVarChaos_Manifold_SphereCapsuleSizeThreshold(TEXT("p.Chaos.Collision.Manifold.SphereCapsuleSizeThreshold"), Chaos_Collision_Manifold_SphereCapsuleSizeThreshold, TEXT(""));

	// When two capsule axes are aligned more than this, generate a multi-point manifold
	FRealSingle Chaos_Collision_Manifold_CapsuleAxisAlignedThreshold = 0.8f;	// About 30deg
	FAutoConsoleVariableRef CVarChaos_Manifold_CapsuleAxisAlignedThreshold(TEXT("p.Chaos.Collision.Manifold.CapsuleAxisAlignedThreshold"), Chaos_Collision_Manifold_CapsuleAxisAlignedThreshold, TEXT(""));

	// When two capsules penetrate by more than this fraction of the radius, generate a multi-point manifold
	FRealSingle Chaos_Collision_Manifold_CapsuleDeepPenetrationFraction = 0.05f;
	FAutoConsoleVariableRef CVarChaos_Manifold_CapsuleDeepPenetrationFraction(TEXT("p.Chaos.Collision.Manifold.CapsuleDeepPenetrationFraction"), Chaos_Collision_Manifold_CapsuleDeepPenetrationFraction, TEXT(""));

	// When two capsule lie on top of each other in an X, the extra manifold points are this fraction of the radius from the primary contact point
	FRealSingle Chaos_Collision_Manifold_CapsuleRadialContactFraction = 0.25f;
	FAutoConsoleVariableRef CVarChaos_Manifold_CapsuleRadialContactFraction(TEXT("p.Chaos.Collision.Manifold.CapsuleRadialContactFraction"), Chaos_Collision_Manifold_CapsuleRadialContactFraction, TEXT(""));

	// When a capsule-convex contact has points closer together than this fraction of radius, ignore one of the contacts
	FRealSingle Chaos_Collision_Manifold_CapsuleMinContactDistanceFraction = 0.1f;
	FAutoConsoleVariableRef CVarChaos_Manifold_CapsuleMinContactDistanceFraction(TEXT("p.Chaos.Collision.Manifold.CapsuleMinContactDistanceFraction"), Chaos_Collision_Manifold_CapsuleMinContactDistanceFraction, TEXT(""));

	FRealSingle Chaos_Collision_Manifold_PlaneContactNormalEpsilon = 0.001f;
	FAutoConsoleVariableRef CVarChaos_Manifold_PlaneContactNormalEpsilon(TEXT("p.Chaos.Collision.Manifold.PlaneContactNormalEpsilon"), Chaos_Collision_Manifold_PlaneContactNormalEpsilon, TEXT("Normal tolerance used to distinguish face contacts from edge-edge contacts"));

	FRealSingle Chaos_Collision_Manifold_TriangleContactNormalThreshold = 0.75f;	// About 40deg
	FAutoConsoleVariableRef CVarChaos_Manifold_TriangleContactNormalThreshold(TEXT("p.Chaos.Collision.Manifold.TriangleNormalThreshold"), Chaos_Collision_Manifold_TriangleContactNormalThreshold, TEXT(""));

	FRealSingle Chaos_Collision_Manifold_EdgeContactNormalThreshold = 0.9f;	// About 25deg
	FAutoConsoleVariableRef CVarChaos_Manifold_EdgeContactNormalThreshold(TEXT("p.Chaos.Collision.Manifold.EdgeNormalThreshold"), Chaos_Collision_Manifold_EdgeContactNormalThreshold, TEXT(""));

	// We use a smaller margin for triangle to convex. Ideally we would have none at all in this case
	// LWC_TODO: This needs to be a larger value for float builds (probably 1)
	FRealSingle Chaos_Collision_Manifold_TriangleConvexMarginMultiplier = 0.5f;
	FAutoConsoleVariableRef CVarChaos_Manifold_TriangleConvexMarginMultipler(TEXT("p.Chaos.Collision.Manifold.TriangleConvexMarginMultiplier"), Chaos_Collision_Manifold_TriangleConvexMarginMultiplier, TEXT(""));

	FRealSingle Chaos_Collision_Manifold_CullDistanceMarginMultiplier = 1.0f;
	FAutoConsoleVariableRef CVarChaosCollisioConvexManifoldCullDistanceMarginMultiplier(TEXT("p.Chaos.Collision.Manifold.CullDistanceMarginMultiplier"), Chaos_Collision_Manifold_CullDistanceMarginMultiplier, TEXT(""));

	FRealSingle Chaos_Collision_Manifold_MinFaceSearchDistance = 1.0f;
	FAutoConsoleVariableRef CVarChaosCollisioConvexManifoldMinFaceSearchDistance(TEXT("p.Chaos.Collision.Manifold.MinFaceSearchDistance"), Chaos_Collision_Manifold_MinFaceSearchDistance, TEXT(""));

	bool ForceOneShotManifoldEdgeEdgeCaseZeroCullDistance = false;
	FAutoConsoleVariableRef CVarForceOneShotManifoldEdgeEdgeCaseZeroCullDistance(TEXT("p.Chaos.Collision.Manifold.ForceOneShotManifoldEdgeEdgeCaseZeroCullDistance"), ForceOneShotManifoldEdgeEdgeCaseZeroCullDistance,
	TEXT("If enabled, if one shot manifold hits edge/edge case, we will force a cull distance of zero. That means edge/edge contacts will be thrown out if separated at all. Only applies to Convex/Convex oneshot impl."));

	bool bChaos_Collision_EnableManifoldGJKReplace = false;
	bool bChaos_Collision_EnableManifoldGJKInject = false;
	FAutoConsoleVariableRef CVarChaos_Collision_EnableManifoldReplace(TEXT("p.Chaos.Collision.EnableManifoldGJKReplace"), bChaos_Collision_EnableManifoldGJKReplace, TEXT(""));
	FAutoConsoleVariableRef CVarChaos_Collision_EnableManifoldInject(TEXT("p.Chaos.Collision.EnableManifoldGJKInject"), bChaos_Collision_EnableManifoldGJKInject, TEXT(""));

	bool bChaos_Manifold_EnableGjkWarmStart = true;
	FAutoConsoleVariableRef CVarChaos_Manifold_EnableGjkWarmStart(TEXT("p.Chaos.Collision.Manifold.EnableGjkWarmStart"), bChaos_Manifold_EnableGjkWarmStart, TEXT(""));

	// See GJKContactPointMargin for comments on why these matter
	// LWC_TODO: These needs to be a larger values for float builds (1.e-3f)
	FRealSingle Chaos_Collision_GJKEpsilon = 1.e-6f;
	FRealSingle Chaos_Collision_EPAEpsilon = 1.e-6f;
	FAutoConsoleVariableRef CVarChaos_Collision_GJKEpsilon(TEXT("p.Chaos.Collision.GJKEpsilon"), Chaos_Collision_GJKEpsilon, TEXT(""));
	FAutoConsoleVariableRef CVarChaos_Collision_EPAEpsilon(TEXT("p.Chaos.Collision.EPAEpsilon"), Chaos_Collision_EPAEpsilon, TEXT(""));

	// Whether to prune spurious edge contacts in trimesh and heighfield contacts (pretty much essential)
	bool bChaos_Collision_EnableEdgePrune = true;
	FRealSingle Chaos_Collision_EdgePrunePlaneDistance = 3.0;
	FAutoConsoleVariableRef CVarChaos_Collision_EnableEdgePrune(TEXT("p.Chaos.Collision.EnableEdgePrune"), bChaos_Collision_EnableEdgePrune, TEXT(""));
	FAutoConsoleVariableRef CVarChaos_Collision_EdgePrunePlaneDistance(TEXT("p.Chaos.Collision.EdgePrunePlaneDistance"), Chaos_Collision_EdgePrunePlaneDistance, TEXT(""));

	bool bChaos_Collision_EnableLargeMeshManifolds = 1;
	FAutoConsoleVariableRef CVarChaos_Collision_EnableLargeMeshManifolds(TEXT("p.Chaos.Collision.EnableLargeMeshManifolds"), bChaos_Collision_EnableLargeMeshManifolds, TEXT("Whether to allow large mesh manifolds for collisions against meshes (required for good behaviour)"));

	FRealSingle Chaos_Collision_MeshContactNormalThreshold = 0.998f;	// ~3deg
	FAutoConsoleVariableRef CVarChaos_Collision_MeshContactNormalThreshold(TEXT("p.Chaos.Collision.MeshContactNormalThreshold"), Chaos_Collision_MeshContactNormalThreshold, TEXT("Treat contact with a dot product between the normal and the triangle face greater than this as face collisions"));

	FRealSingle Chaos_Collision_MeshContactNormalRejectionThreshold = 0.7f;	// ~45deg
	FAutoConsoleVariableRef CVarChaos_Collision_MeshContactNormalRejectionThreshold(TEXT("p.Chaos.Collision.MeshContactNormalRejectionThreshold"), Chaos_Collision_MeshContactNormalRejectionThreshold, TEXT("Don't correct edge and vertex normals if they are beyond the valid range by more than this"));

	bool bChaos_Collision_MeshManifoldSortByDistance = false;
	FAutoConsoleVariableRef CVarChaos_Collision_LargeMeshManifoldSortByDistance(TEXT("p.Chaos.Collision.SortMeshManifoldByDistance"), bChaos_Collision_MeshManifoldSortByDistance, TEXT("Sort large mesh manifold points by |RxN| for improved solver stability (less rotation in first iteration)"));

	int32 Chaos_Collision_MeshManifoldHashSize = 256;
	FAutoConsoleVariableRef CVarChaos_Collision_MeshManifoldHashSize(TEXT("p.Chaos.Collision.MeshManifoldHashSize"), Chaos_Collision_MeshManifoldHashSize, TEXT("Hash table size to use in vertex and edge maps in convex-mesh collision"));

	// @todo(chaos): Temp while we test the new convex-mesh collision optimizations
	bool bChaos_Collision_EnableMeshManifoldOptimizedLoop = true;
	bool bChaos_Collision_EnableMeshManifoldOptimizedLoop_TriMesh = true;
	FAutoConsoleVariableRef CVarChaos_Collision_EnableMeshManifoldOptimizedLoop(TEXT("p.Chaos.Collision.EnableMeshManifoldOptimizedLoop"), bChaos_Collision_EnableMeshManifoldOptimizedLoop, TEXT(""));
	FAutoConsoleVariableRef CVarChaos_Collision_EnableMeshManifoldOptimizedLoop_TriMesh(TEXT("p.Chaos.Collision.EnableMeshManifoldOptimizedLoopTriMesh"), bChaos_Collision_EnableMeshManifoldOptimizedLoop_TriMesh, TEXT(""));

	// MACD uses the non-MACD path when the shape is outside the triangle plane
	bool bChaos_Collision_EnableMACDFallback = false;
	FAutoConsoleVariableRef CVarChaos_Collision_EnableMACDFallback(TEXT("p.Chaos.Collision.EnableMACDFallback"), bChaos_Collision_EnableMACDFallback, TEXT(""));

	// We have 2 MACD algorithms to test, selected by bChaos_Collision_EnableMACDPreManifoldFix
	// false: Use the original algorithm which generates a manifold for every triangle and then uses the mesh information to correct the normal of each manifold point.
	// true: Use the new algorithm which generates the closest feature and fixes its normal based on the mesh information, and then generates a manifold based on that.
	// The second method does a much better job at handling deep collisions where a single convex-triangle contact might generate a contact that pushes the convex out
	// of the triangle edge, even though the triangle is part of a mesh and pushing out of the edge is not a valid option. However this option is also more likely to 
	// generate ghost collisions with nearby faces when moving very fast.
	// @todo(chaos): the second method has problems with complicated meshes and need to be disabled.
	bool bChaos_Collision_EnableMACDPreManifoldFix = false;
	FAutoConsoleVariableRef CVarChaos_Collision_EnableMACDPreManifoldFix(TEXT("p.Chaos.Collision.EnableMACDPreManifoldFix"), bChaos_Collision_EnableMACDPreManifoldFix, TEXT(""));

	// Whether to use the new index-less GJK. 
	// @todo(chaos): This should be removed once soaked for a bit (enabled 7 June 2022)
	bool bChaos_Collision_UseGJK2 = false;
	FAutoConsoleVariableRef CVarChaos_Collision_UseGJK2(TEXT("p.Chaos.Collision.UseGJK2"), bChaos_Collision_UseGJK2, TEXT(""));

	bool bChaos_Collision_OneSidedTriangleMesh = true;
	bool bChaos_Collision_OneSidedHeightField = true;
	FAutoConsoleVariableRef CVarChaos_Collision_OneSidedTriangleMesh(TEXT("p.Chaos.Collision.OneSidedTriangleMesh"), bChaos_Collision_OneSidedTriangleMesh, TEXT(""));
	FAutoConsoleVariableRef CVarChaos_Collision_OneSidedHeightfield(TEXT("p.Chaos.Collision.OneSidedHeightField"), bChaos_Collision_OneSidedHeightField, TEXT(""));

	// Ueed to reject contacts against tri meshes
	FRealSingle Chaos_Collision_TriMeshPhiToleranceScale = 1.0f;	// A multipler on cull distance. Points farther than this from the deepest point are ignored
	FRealSingle Chaos_Collision_TriMeshDistanceTolerance = 0.1f;	// Points closer than this to a deeper point are ignored
	FAutoConsoleVariableRef CVarChaos_Collision_TriMeshDistanceolerance(TEXT("p.Chaos.Collision.TriangeMeshDistanceTolerance"), Chaos_Collision_TriMeshDistanceTolerance, TEXT(""));
	FAutoConsoleVariableRef CVarChaos_Collision_TriMeshPhiToleranceScale(TEXT("p.Chaos.Collision.TriangeMeshPhiToleranceScale"), Chaos_Collision_TriMeshPhiToleranceScale, TEXT(""));

	bool bChaos_Collision_UseCapsuleTriMesh2 = true;
	FAutoConsoleVariableRef CVarChaos_Collision_UseCapsuleTriMesh2(TEXT("p.Chaos.Collision.UseCapsuleTriMesh2"), bChaos_Collision_UseCapsuleTriMesh2, TEXT(""));
	
	// @todo(chaos): (2) is the intended solution. The others are for rollback if blocking bugs are found. Remove this once (2) is tested
	// 0: Use GJK + EPA to generate closest feature, generate multi-point manifold from its normal, and then fix manifold based on mesh topology
	// 1: Use SAT to generate closest feature, generate multi-point manifold from its normal, and then fix manifold based on mesh topology
	// 2: Use GJK + SAT to generate closest feature, then fix its normal based on mesh topology, then generate the multi-point manifold
	int32 Chaos_Collision_ConvexTriMeshMode = 1;
	FAutoConsoleVariableRef CVarChaos_Collision_UseConvexTriMesh2(TEXT("p.Chaos.Collision.ConvexTriMeshMode"), Chaos_Collision_ConvexTriMeshMode, TEXT(""));

	// Enable backface culling in ConstructConvexTriangleOneShotManifold3
	bool bChaos_Collision_ConvexTriMeshBackFaceCull = true;
	FAutoConsoleVariableRef CVarChaos_Collision_ConvexTriMeshBackFaceCull(TEXT("p.Chaos.Collision.ConvexTriMeshBackFaceCull"), bChaos_Collision_ConvexTriMeshBackFaceCull, TEXT(""));

	// Enable convex-inside-triangle culling in ConstructConvexTriangleOneShotManifold3
	bool bChaos_Collision_ConvexTriMeshInsideCull = true;
	FAutoConsoleVariableRef CVarChaos_Collision_ConvexTriMeshInsideCull(TEXT("p.Chaos.Collision.ConvexTriMeshInsideCull"), bChaos_Collision_ConvexTriMeshInsideCull, TEXT(""));

	// true: use GJK and then SAT if the shapes overlap
	// false: use GJK and then EPA if the shapes overlap
	bool bChaos_Collision_UseConvexTriangleGJKSAT = true;
	FAutoConsoleVariableRef CVarChaos_Collision_UseConvexTriangleSAT(TEXT("p.Chaos.Collision.UseConvexTriangleGJKSAT"), bChaos_Collision_UseConvexTriangleGJKSAT, TEXT(""));

	// Whether to sort sphere/capsule/convex vs mesh contacts by depth. Doing so leads to more consistent behaviour but is more expensive
	bool bChaos_Collision_ConvexTriMeshSortByPhi = false;
	FAutoConsoleVariableRef CVarChaos_Collision_ConvexTriMeshSortByPhi(TEXT("p.Chaos.Collision.ConvexTriMeshSortByPhi"), bChaos_Collision_ConvexTriMeshSortByPhi, TEXT(""));

	// Whether to sort sphere/capsule/convex vs mesh contacts by distance from center of mass in the contact plane. Slow but better for convergence.
	bool bChaos_Collision_ConvexTriMeshSortByDistance = false;
	FAutoConsoleVariableRef CVarChaos_Collision_ConvexTriMeshSortByDistance(TEXT("p.Chaos.Collision.ConvexTriMeshSortByDistance"), bChaos_Collision_ConvexTriMeshSortByDistance, TEXT(""));

	namespace Collisions
	{
		// Forward delarations we need from CollisionRestitution.cpp

		FContactPoint BoxBoxContactPoint(const FImplicitBox3& Box1, const FImplicitBox3& Box2, const FRigidTransform3& Box1TM, const FRigidTransform3& Box2TM);

		//////////////////////////
		// Box Box
		//////////////////////////

		// This function will clip the input vertices by a reference shape's planes (Specified by ClippingAxis and Distance for an AABB)
		// more vertices may be added to outputVertexBuffer by this function
		// This is the core of the Sutherland-Hodgman algorithm
		uint32 BoxBoxClipVerticesAgainstPlane(const FVec3* InputVertexBuffer, FVec3* outputVertexBuffer, uint32 ClipPointCount, int32 ClippingAxis, FReal Distance)
		{

			auto CalculateIntersect = [=](const FVec3& Point1, const FVec3& Point2) -> FVec3
			{
				// Only needs to be valid if the line connecting Point1 with Point2 actually intersects
				FVec3 Result;

				FReal Denominator = Point2[ClippingAxis] - Point1[ClippingAxis];  // Can be negative
				if (FMath::Abs(Denominator) < UE_SMALL_NUMBER)
				{
					Result = Point1;
				}
				else
				{
					FReal Alpha = (Distance - Point1[ClippingAxis]) / Denominator;
					Result = FMath::Lerp(Point1, Point2, Alpha);
				}
				Result[ClippingAxis] = Distance; // For Robustness
				return Result;
			};

			auto InsideClipFace = [=](const FVec3& Point) -> bool
			{
				// The sign of Distance encodes which plane we are using
				if (Distance >= 0)
				{
					return Point[ClippingAxis] <= Distance;
				}
				return Point[ClippingAxis] >= Distance;
			};

			uint32 NewClipPointCount = 0;
			const uint32 MaxNumberOfPoints = 8;

			for (uint32 ClipPointIndex = 0; ClipPointIndex < ClipPointCount; ClipPointIndex++)
			{
				FVec3 CurrentClipPoint = InputVertexBuffer[ClipPointIndex];
				FVec3 PrevClipPoint = InputVertexBuffer[(ClipPointIndex + ClipPointCount - 1) % ClipPointCount];
				FVec3 InterSect = CalculateIntersect(PrevClipPoint, CurrentClipPoint);

				if (InsideClipFace(CurrentClipPoint))
				{
					if (!InsideClipFace(PrevClipPoint))
					{
						outputVertexBuffer[NewClipPointCount++] = InterSect;
						if (NewClipPointCount >= MaxNumberOfPoints)
						{
							break;
						}
					}
					outputVertexBuffer[NewClipPointCount++] = CurrentClipPoint;
				}
				else if (InsideClipFace(PrevClipPoint))
				{
					outputVertexBuffer[NewClipPointCount++] = InterSect;
				}

				if (NewClipPointCount >= MaxNumberOfPoints)
				{
					break;
				}
			}

			return NewClipPointCount;
		}

		void ConstructBoxBoxOneShotManifold(
			const FImplicitBox3& Box1,
			const FRigidTransform3& Box1Transform, //world
			const FImplicitBox3& Box2,
			const FRigidTransform3& Box2Transform, //world
			const FReal Dt,
			FPBDCollisionConstraint& Constraint)
		{
			ConstructConvexConvexOneShotManifold(Box1, Box1Transform, Box2, Box2Transform, Dt, Constraint);
		}

		/////////////////////////////
		/// General Convexes
		/////////////////////////////

		// Reduce the number of contact points (in place)
		// Prerequisites to calling this function:
		// The points should be in a reference frame such that the z-axis is the in the direction of the separation vector
		uint32 ReduceManifoldContactPoints(FVec3* Points, uint32 PointCount)
		{
			uint32 OutPointCount = 0;
			if (PointCount <= 4)
				return PointCount;

			// Point 1) Find the deepest contact point
			{
				uint32 DeepestPointIndex = 0;
				FReal DeepestPointPhi = FLT_MAX;
				for (uint32 PointIndex = 0; PointIndex < PointCount; PointIndex++)
				{
					if (Points[PointIndex].Z < DeepestPointPhi)
					{
						DeepestPointIndex = PointIndex;
						DeepestPointPhi = Points[PointIndex].Z;
					}
				}
				// Deepest point will be our first output point
				Swap(Points[0], Points[DeepestPointIndex]);
				++OutPointCount;
			}

			// Point 2) Find the point with the largest distance to the deepest contact point (projected onto the separation plane)
			{
				uint32 FarthestPointIndex = 1;
				FReal FarthestPointDistanceSQR = -1.0f;
				for (uint32 PointIndex = 1; PointIndex < PointCount; PointIndex++)
				{
					FReal PointAToPointBSizeSQR = (Points[PointIndex] - Points[0]).SizeSquared2D();
					if (PointAToPointBSizeSQR > FarthestPointDistanceSQR)
					{
						FarthestPointIndex = PointIndex;
						FarthestPointDistanceSQR = PointAToPointBSizeSQR;
					}
				}
				// Farthest point will be added now
				Swap(Points[1], Points[FarthestPointIndex]);
				++OutPointCount;
			}

			// Point 3) Largest triangle area
			{
				uint32 LargestTrianglePointIndex = 2;
				FReal LargestTrianglePointSignedArea = 0.0f; // This will actually be double the signed area
				FVec3 P0to1 = Points[1] - Points[0];
				for (uint32 PointIndex = 2; PointIndex < PointCount; PointIndex++)
				{
					FReal TriangleSignedArea = (FVec3::CrossProduct(P0to1, Points[PointIndex] - Points[0])).Z; // Dot in direction of separation vector
					if (FMath::Abs(TriangleSignedArea) > FMath::Abs(LargestTrianglePointSignedArea))
					{
						LargestTrianglePointIndex = PointIndex;
						LargestTrianglePointSignedArea = TriangleSignedArea;
					}
				}
				// Point causing the largest triangle will be added now
				Swap(Points[2], Points[LargestTrianglePointIndex]);
				++OutPointCount;
				// Ensure the winding order is consistent
				if (LargestTrianglePointSignedArea < 0)
				{
					Swap(Points[0], Points[1]);
				}
			}

			// Point 4) Find the largest triangle connecting with our current triangle
			{
				uint32 LargestTrianglePointIndex = 3;
				FReal LargestPositiveTrianglePointSignedArea = 0.0f;
				for (uint32 PointIndex = 3; PointIndex < PointCount; PointIndex++)
				{
					for (uint32 EdgeIndex = 0; EdgeIndex < 3; EdgeIndex++)
					{
						FReal TriangleSignedArea = (FVec3::CrossProduct(Points[PointIndex] - Points[EdgeIndex], Points[(EdgeIndex + 1) % 3] - Points[EdgeIndex])).Z; // Dot in direction of separation vector
						if (TriangleSignedArea > LargestPositiveTrianglePointSignedArea)
						{
							LargestTrianglePointIndex = PointIndex;
							LargestPositiveTrianglePointSignedArea = TriangleSignedArea;
						}
					}
				}
				// Point causing the largest positive triangle area will be added now
				Swap(Points[3], Points[LargestTrianglePointIndex]);
				++OutPointCount;
			}

			return OutPointCount; // This should always be 4
		}

		// This function will clip the input vertices by a reference shape's planes
		// more vertices may be added to outputVertexBuffer by this function
		// This is the core of the Sutherland-Hodgman algorithm
		// Plane Normals face outwards 
		uint32 ClipVerticesAgainstPlane(const FVec3* InputVertexBuffer, FVec3* OutputVertexBuffer, const uint32 ClipPointCount, const uint32 MaxNumberOfOutputPoints, const FVec3 ClippingPlaneNormal, const FReal PlaneDistance)
		{
			check(ClipPointCount > 0);

			uint32 NewClipPointCount = 0;

			FVec3 PrevClipPoint;
			FReal PrevClipPointDotNormal;
			FVec3 CurrentClipPoint = InputVertexBuffer[ClipPointCount - 1];;
			FReal CurrentClipPointDotNormal = FVec3::DotProduct(CurrentClipPoint, ClippingPlaneNormal);
			const FReal PlaneClipDistance = PlaneDistance + PlaneDistance * UE_SMALL_NUMBER;

			auto CalculateIntersect = [&PrevClipPoint, &CurrentClipPoint, &PlaneDistance](const FReal Dot1, const FReal Dot2)
			{
				const FReal Denominator = Dot2 - Dot1; // Can be negative
				if (FMath::Abs(Denominator) < UE_SMALL_NUMBER)
				{
					return PrevClipPoint;
				}
				else
				{
					const FReal Alpha = (PlaneDistance - Dot1) / Denominator;
					return FVec3::Lerp(PrevClipPoint, CurrentClipPoint, Alpha);
				}
			};

			for (uint32 ClipPointIndex = 0; ClipPointIndex < ClipPointCount; ClipPointIndex++)
			{
				PrevClipPoint = CurrentClipPoint;
				PrevClipPointDotNormal = CurrentClipPointDotNormal;
				CurrentClipPoint = InputVertexBuffer[ClipPointIndex];
				CurrentClipPointDotNormal = FVec3::DotProduct(CurrentClipPoint, ClippingPlaneNormal);

				// Epsilon is there so that previously clipped points will still be inside the plane
				if (CurrentClipPointDotNormal <= PlaneClipDistance)
				{
					if (PrevClipPointDotNormal > PlaneClipDistance)
					{
						OutputVertexBuffer[NewClipPointCount++] = CalculateIntersect(PrevClipPointDotNormal, CurrentClipPointDotNormal);
						if (NewClipPointCount >= MaxNumberOfOutputPoints)
						{
							break;
						}
					}
					OutputVertexBuffer[NewClipPointCount++] = CurrentClipPoint;
				}
				else if (PrevClipPointDotNormal < PlaneClipDistance)
				{
					OutputVertexBuffer[NewClipPointCount++] = CalculateIntersect(PrevClipPointDotNormal, CurrentClipPointDotNormal);
				}

				if (NewClipPointCount >= MaxNumberOfOutputPoints)
				{
					break;
				}
			}

			return NewClipPointCount;
		}

		template <typename ConvexImplicitType1, typename ConvexImplicitType2>
		FVec3* GenerateConvexManifoldClippedVertices(
			const ConvexImplicitType1& RefConvex,
			const ConvexImplicitType2& OtherConvex,
			const FRigidTransform3& OtherToRefTransform,
			const int32 RefPlaneIndex,
			const int32 OtherPlaneIndex,
			const FVec3& RefPlaneNormal,
			FVec3* VertexBuffer1,
			FVec3* VertexBuffer2,
			uint32& ContactPointCount,	// InOut
			const uint32 MaxContactPointCount
		)
		{
			// Populate the clipped vertices by the other face's vertices
			const int32 OtherConvexFaceVerticesNum = OtherConvex.NumPlaneVertices(OtherPlaneIndex);
			ContactPointCount = FMath::Min(OtherConvexFaceVerticesNum, (int32)MaxContactPointCount); // Number of face vertices
			for (int32 VertexIndex = 0; VertexIndex < (int32)ContactPointCount; ++VertexIndex)
			{
				// Todo Check for Grey code
				const FVec3 OtherVertex = OtherConvex.GetVertex(OtherConvex.GetPlaneVertex(OtherPlaneIndex, VertexIndex));
				VertexBuffer1[VertexIndex] = OtherToRefTransform.TransformPositionNoScale(OtherVertex);
			}

			// Now clip against all planes that belong to the reference plane's, edges
			// Note winding order matters here, and we have to handle negative scales
			const FReal RefWindingOrder = RefConvex.GetWindingOrder();
			const int32 RefConvexFaceVerticesNum = RefConvex.NumPlaneVertices(RefPlaneIndex);
			int32 ClippingPlaneCount = RefConvexFaceVerticesNum;
			FVec3 PrevPoint = RefConvex.GetVertex(RefConvex.GetPlaneVertex(RefPlaneIndex, ClippingPlaneCount - 1));
			for (int32 ClippingPlaneIndex = 0; (ClippingPlaneIndex < ClippingPlaneCount) && (ContactPointCount > 1); ++ClippingPlaneIndex)
			{
				FVec3 CurrentPoint = RefConvex.GetVertex(RefConvex.GetPlaneVertex(RefPlaneIndex, ClippingPlaneIndex));
				FVec3 ClippingPlaneNormal = RefWindingOrder * FVec3::CrossProduct(RefPlaneNormal, PrevPoint - CurrentPoint);
				ClippingPlaneNormal.SafeNormalize();
				ContactPointCount = ClipVerticesAgainstPlane(VertexBuffer1, VertexBuffer2, ContactPointCount, MaxContactPointCount, ClippingPlaneNormal, FVec3::DotProduct(CurrentPoint, ClippingPlaneNormal));
				Swap(VertexBuffer1, VertexBuffer2); // VertexBuffer1 will now point to the latest
				PrevPoint = CurrentPoint;
			}

			return VertexBuffer1;
		}

		template <typename ConvexImplicitType>
		FVec3* GenerateLineConvexManifoldClippedVerticesSameSpace(
			const ConvexImplicitType& Convex,
			const TSegment<FReal>& Segment,
			const int32 ConvexPlaneIndex,
			const FVec3& PlaneNormal,
			FVec3* VertexBuffer,
			int32& ContactPointCount,	// InOut
			const int32 MaxContactPointCount,
			const FReal PlaneTolerance)
		{
			check(MaxContactPointCount >= 2);

			const FReal DistanceEpsilon = FReal(UE_KINDA_SMALL_NUMBER);

			// Populate the clipped vertices by the line segment vertices
			ContactPointCount = 2;
			VertexBuffer[0] = Segment.GetX1();
			VertexBuffer[1] = Segment.GetX2();

			// Now clip against all edge planes that belong to the convex face
			// Note winding order matters here, and we have to handle negative scales
			const FReal WindingOrder = Convex.GetWindingOrder();
			const int32 ConvexFaceVerticesNum = Convex.NumPlaneVertices(ConvexPlaneIndex);
			int32 ClippingPlaneCount = ConvexFaceVerticesNum;
			FVec3 PrevPoint = Convex.GetVertex(Convex.GetPlaneVertex(ConvexPlaneIndex, ClippingPlaneCount - 1));
			for (int32 ClippingPlaneIndex = 0; (ClippingPlaneIndex < ClippingPlaneCount) && (ContactPointCount > 1); ++ClippingPlaneIndex)
			{
				const FVec3 CurrentPoint = Convex.GetVertex(Convex.GetPlaneVertex(ConvexPlaneIndex, ClippingPlaneIndex));
				const FVec3 ClippingPlaneNormal = WindingOrder * FVec3::CrossProduct(PlaneNormal, PrevPoint - CurrentPoint).GetSafeNormal();
				PrevPoint = CurrentPoint;

				// Distance to plane (+ve outside, -ve inside)
				const FReal Dist0 = FVec3::DotProduct(VertexBuffer[0] - CurrentPoint, ClippingPlaneNormal);
				const FReal Dist1 = FVec3::DotProduct(VertexBuffer[1] - CurrentPoint, ClippingPlaneNormal);
				if ((Dist0 > PlaneTolerance) && (Dist1 > PlaneTolerance))
				{
					// Both points are outside, so no contacts
					ContactPointCount = 0;
					break;
				}
				else if ((Dist0 > PlaneTolerance) && (Dist1 < PlaneTolerance))
				{
					// First point is outside, second inside
					const FReal Alpha = Dist1 / (Dist1 - Dist0);
					VertexBuffer[0] = VertexBuffer[1] + Alpha * (VertexBuffer[0] - VertexBuffer[1]);
				}
				else if ((Dist1 > PlaneTolerance) && (Dist0 < PlaneTolerance))
				{
					// Second point is outside, first inside
					const FReal Alpha = Dist0 / (Dist0 - Dist1);
					VertexBuffer[1] = VertexBuffer[0] + Alpha * (VertexBuffer[1] - VertexBuffer[0]);
				}
			}

			return VertexBuffer;
		}

		// Use GJK to find the closest points (or shallowest penetrating points) on two convex shapes usingthe specified margin
		// @todo(chaos): dedupe from GJKContactPoint in CollisionResolution.cpp
		template <typename GeometryA, typename GeometryB>
		FContactPoint GJKContactPointMargin(const GeometryA& A, const GeometryB& B, const FRigidTransform3& BToATM, FReal MarginA, FReal MarginB, FGJKSimplexData& InOutGjkWarmStartData, FReal& OutMaxMarginDelta, int32& VertexIndexA, int32& VertexIndexB)
		{
			SCOPE_CYCLE_COUNTER_MANIFOLD_GJK();

			FContactPoint Contact;

			FReal Penetration;
			FVec3 ClosestA, ClosestB, NormalA, NormalB;

			// GJK and EPA tolerances.
			// The GJK tolerance controls the separating distance at which GJK hands over to EPA. We need to be
			// able to normalize vectors of size epsilon with low error to avoid bad normals. 
			// This is a problem for floats with Epsilon < 1e-3f. For doubles it can be nmuch smaller although this
			// increases how many iterations we need when quadratic shapes are involved.
			// The EPA tolerance is used to determine whether a point projects inside a face on the simplezx.
			// EPA is also sensitive to the GJK tolerance, because it assumes the simplex from GJK contains the origin
			// but if it does not and the origin also does not project onto any simplex planes, we can end up with
			// a bad case in EPA where it rejects the correct face and returns some arbitrary other face. We need an
			// EPA Tolerance of about 1e-6 to avoid this in our test cases. It may need to be smaller.
			//const FReal Epsilon = 3.e-3f;	// original value for float implementation
			const FReal GJKEpsilon = Chaos_Collision_GJKEpsilon;
			const FReal EPAEpsilon = Chaos_Collision_EPAEpsilon;

			const TGJKCoreShape<GeometryA> AWithMargin(A, MarginA);
			const TGJKCoreShape<GeometryB> BWithMargin(B, MarginB);

			bool bHaveContact = false;
			if (bChaos_Collision_UseGJK2)
			{
				bHaveContact = GJKPenetrationWarmStartable2(AWithMargin, BWithMargin, BToATM, Penetration, ClosestA, ClosestB, NormalA, NormalB, VertexIndexA, VertexIndexB, InOutGjkWarmStartData, OutMaxMarginDelta, GJKEpsilon, EPAEpsilon);
			}
			else
			{
				bHaveContact = GJKPenetrationWarmStartable(AWithMargin, BWithMargin, BToATM, Penetration, ClosestA, ClosestB, NormalA, NormalB, VertexIndexA, VertexIndexB, InOutGjkWarmStartData, OutMaxMarginDelta, GJKEpsilon, EPAEpsilon);
			}

			if (bHaveContact)
			{
				Contact.ShapeContactPoints[0] = ClosestA;
				Contact.ShapeContactPoints[1] = ClosestB;
				Contact.ShapeContactNormal = -NormalB;	// We want normal pointing from B to A
				Contact.Phi = -Penetration;
			}

			return Contact;
		}

		// GJK contact point between two GJKShape wrapped implicits
		// This assumes that the both shapes are in the same space
		template <typename GJKShapeA, typename GJKShapeB>
		FContactPoint GJKContactPointSameSpace(const GJKShapeA& A, const GJKShapeB& B, FReal& OutMaxMarginDelta, int32& VertexIndexA, int32& VertexIndexB, const FVec3 InitialGJKDir = FVec3(-1,0,0))
		{
			SCOPE_CYCLE_COUNTER_MANIFOLD_GJK();

			FContactPoint Contact;

			FReal Penetration;
			FVec3 ClosestA, ClosestB, Normal;

			// GJK and EPA tolerances. See comments in GJKContactPointMargin
			const FReal GJKEpsilon = Chaos_Collision_GJKEpsilon;
			const FReal EPAEpsilon = Chaos_Collision_EPAEpsilon;

			bool bHaveContact = false;
			if (bChaos_Collision_UseGJK2)
			{
				bHaveContact = GJKPenetrationSameSpace2(A, B, Penetration, ClosestA, ClosestB, Normal, VertexIndexA, VertexIndexB, OutMaxMarginDelta, InitialGJKDir, GJKEpsilon, EPAEpsilon);
			}
			else
			{
				bHaveContact = GJKPenetrationSameSpace(A, B, Penetration, ClosestA, ClosestB, Normal, VertexIndexA, VertexIndexB, OutMaxMarginDelta, InitialGJKDir, GJKEpsilon, EPAEpsilon);
			}

			if (bHaveContact)
			{
				Contact.ShapeContactPoints[0] = ClosestA;
				Contact.ShapeContactPoints[1] = ClosestB;
				Contact.ShapeContactNormal = -Normal;	// We want normal pointing from B to A
				Contact.Phi = -Penetration;
			}

			return Contact;
		}

		// Find the the most opposing plane given a position and a direction
		template <typename ConvexImplicitType>
		void FindBestPlane(
			const ConvexImplicitType& Convex,
			const FVec3& X,
			const FVec3& N,
			const FReal MaxDistance,
			const int32 PlaneIndex,
			int32& BestPlaneIndex,
			FReal& BestPlaneDot)
		{
			const TPlaneConcrete<FReal, 3> Plane = Convex.GetPlane(PlaneIndex);
				
			// Reject planes farther than MaxDistance
			const FReal PlaneDistance = Plane.SignedDistance(X);
			if (FMath::Abs(PlaneDistance) <= MaxDistance)
			{
				// Ignore planes that do not oppose N
				const FReal PlaneNormalDotN = FVec3::DotProduct(N, Plane.Normal());
				if (PlaneNormalDotN <= -UE_SMALL_NUMBER)
				{
					// Keep the most opposing plane
					if (PlaneNormalDotN < BestPlaneDot)
					{
						BestPlaneDot = PlaneNormalDotN;
						BestPlaneIndex = PlaneIndex;
					}
				}
			}
		}

		// Specialization for scaled convex. Avoids the instantiation of a scaled plane object
		// which almost doubles the cost of the function.
		void FindBestPlaneScaledConvex(
			const TImplicitObjectScaled<FConvex>& ScaledConvex,
			const FConvex::FVec3Type& X,
			const FConvex::FVec3Type& N,
			const FConvex::FVec3Type& Scale,
			const FConvex::FVec3Type& ScaleInv,
			const FConvex::FRealType MaxDistance,
			const int32 PlaneIndex,
			int32& OutBestPlaneIndex,
			FConvex::FRealType& InOutBestPlaneDot)
		{
			using FConvexReal = FConvex::FRealType;
			using FConvexVec3 = FConvex::FVec3Type;
			using FConvexPlane = FConvex::FPlaneType;

			const FConvex* UnscaledConvex = ScaledConvex.GetInnerObject()->template GetObject<FConvex>();
			const FConvexPlane& UnscaledPlane = UnscaledConvex->GetPlaneRaw(PlaneIndex);

			const FConvexVec3 ScaledPlaneX = UnscaledPlane.X() * Scale;
			FConvexVec3 ScaledPlaneN = UnscaledPlane.Normal() * ScaleInv;

			if (ScaledPlaneN.Normalize())
			{
				// Reject planes farther than MaxDistance
				const FConvexReal PlaneDistance = FConvexVec3::DotProduct(X - ScaledPlaneX, ScaledPlaneN);
				if (FMath::Abs(PlaneDistance) <= MaxDistance)
				{
					// Ignore planes that do not oppose N
					const FConvexReal PlaneNormalDotN = FConvexVec3::DotProduct(N, ScaledPlaneN);
					if (PlaneNormalDotN <= -UE_SMALL_NUMBER)
					{
						// Keep the most opposing plane
						if (PlaneNormalDotN < InOutBestPlaneDot)
						{
							InOutBestPlaneDot = PlaneNormalDotN;
							OutBestPlaneIndex = PlaneIndex;
						}
					}
				}
			}
		}
		
		// SelectContactPlane was failing. This issue was tracked down to a degenerate simplex in GJK, but we may need this again one day...
		template<typename ConvexImplicitType>
		void CheckPlaneIndex(const int32 PlaneIndex, const ConvexImplicitType& Convex, const FVec3 X, const FVec3 N, const FReal MaxDistance, const int32 VertexIndex)
		{
			CHAOS_COLLISIONERROR_CLOG(PlaneIndex == INDEX_NONE, TEXT("SelectContactPlane: Invalid PlaneIndex %d, X: [%f, %f, %f], N: [%f, %f, %f], MaxDistance: %f, VertexIndex: %d\n%s"), PlaneIndex, X.X, X.Y, X.Z, N.X, N.Y, N.Z, MaxDistance, VertexIndex, *Convex.ToString());
			CHAOS_COLLISIONERROR_ENSURE(PlaneIndex != INDEX_NONE);
		}

		// Select one of the planes on the convex to use as the contact plane, given an estimated contact position and opposing 
		// normal from GJK with margins (which gives the shapes rounded corners/edges).
		template <typename ConvexImplicitType>
		int32 SelectContactPlane(
			const ConvexImplicitType& Convex,
			const FVec3 X,
			const FVec3 N,
			const FReal InMaxDistance,
			const int32 VertexIndex)
		{
			// Handle InMaxDistance = 0. We expect that the X is actually on the surface in this case, so the search distance just needs to be some reasonable tolerance.
			// @todo(chaos): this should probable be dependent on the size of the objects...
			const FReal MinFaceSearchDistance = Chaos_Collision_Manifold_MinFaceSearchDistance;
			const FReal MaxDistance = FMath::Max(InMaxDistance, MinFaceSearchDistance);

			int32 BestPlaneIndex = INDEX_NONE;
			FReal BestPlaneDot = 1.0f;
			{
				int32 PlaneIndices[3] = {INDEX_NONE, INDEX_NONE, INDEX_NONE};
				int32 NumPlanes = Convex.GetVertexPlanes3(VertexIndex, PlaneIndices[0], PlaneIndices[1], PlaneIndices[2]);

				// If we have more than 3 planes we iterate over the full set of planes since it is faster than using the half edge structure
				if(NumPlanes > 3)
				{
					NumPlanes = Convex.NumPlanes();
					for (int32 PlaneIndex = 0; PlaneIndex < NumPlanes; ++PlaneIndex)
					{
						FindBestPlane(Convex, X, N, MaxDistance, PlaneIndex, BestPlaneIndex, BestPlaneDot);
					}
				}
				// Otherwise we iterate over the cached planes
				else
				{
					for (int32 PlaneIndex = 0; PlaneIndex < NumPlanes; ++PlaneIndex)
					{
						FindBestPlane(Convex, X, N, MaxDistance, PlaneIndices[PlaneIndex], BestPlaneIndex, BestPlaneDot);
					}
				}
			}
			
			// Malformed convexes or half-spaces or capsules could have all planes rejected above.
			// If that happens, select the most opposing plane including those that
			// may point the same direction as N. 
			if (BestPlaneIndex == INDEX_NONE)
			{
				// This always returns a valid plane.
				BestPlaneIndex = Convex.GetMostOpposingPlane(N);
			}

			CheckPlaneIndex(BestPlaneIndex, Convex, X, N, InMaxDistance, VertexIndex);
			return BestPlaneIndex;
		}

		template <>
		int32 SelectContactPlane<TImplicitObjectScaled<FConvex>>(
			const TImplicitObjectScaled<FConvex>& ScaledConvex,
			const FVec3 InX,
			const FVec3 InN,
			const FReal InMaxDistance,
			const int32 VertexIndex)
		{
			using FConvexReal = FConvex::FRealType;
			using FConvexVec3 = FConvex::FVec3Type;
			using FConvexPlane = FConvex::FPlaneType;

			// Handle InMaxDistance = 0. We expect that the X is actually on the surface in this case, so the search distance just needs to be some reasonable tolerance.
			// @todo(chaos): this should probable be dependent on the size of the objects...
			const FConvexReal MinFaceSearchDistance = Chaos_Collision_Manifold_MinFaceSearchDistance;
			const FConvexReal MaxDistance = FMath::Max(FConvexReal(InMaxDistance), MinFaceSearchDistance);

			// LWC_TODO: Precision Loss
			// Scale precision is ok as long as we only support large positions and not large sizes...
			// N precision is ok with floats (normalized)
			// X precision is ok as long as we don't try to find the exact separation of objects that are separated by LWC distances
			// which we don't in collision detection (objects must pass bounds checks first).
			const FConvexVec3 Scale = FConvexVec3(ScaledConvex.GetScale());
			const FConvexVec3 ScaleInv = FConvexVec3(ScaledConvex.GetInvScale());
			const FConvexVec3 X = FConvexVec3(InX);
			const FConvexVec3 N = FConvexVec3(InN);

			int32 BestPlaneIndex = INDEX_NONE;
			FConvexReal BestPlaneDot = 1.0f;
			{
				int32 PlaneIndices[3] = { INDEX_NONE, INDEX_NONE, INDEX_NONE };
				int32 NumPlanes = ScaledConvex.GetVertexPlanes3(VertexIndex, PlaneIndices[0], PlaneIndices[1], PlaneIndices[2]);

				// If we have more than 3 planes we iterate over the full set of planes since it is faster than using the half edge structure
				if (NumPlanes > 3)
				{
					NumPlanes = ScaledConvex.NumPlanes();
					for (int32 PlaneIndex = 0; PlaneIndex < NumPlanes; ++PlaneIndex)
					{
						FindBestPlaneScaledConvex(ScaledConvex, X, N, Scale, ScaleInv, MaxDistance, PlaneIndex, BestPlaneIndex, BestPlaneDot);
					}
				}
				// Otherwise we iterate over the cached planes
				else
				{
					for (int32 PlaneIndex = 0; PlaneIndex < NumPlanes; ++PlaneIndex)
					{
						FindBestPlaneScaledConvex(ScaledConvex, X, N, Scale, ScaleInv, MaxDistance, PlaneIndices[PlaneIndex], BestPlaneIndex, BestPlaneDot);
					}
				}
			}

			// Malformed convexes or half-spaces or capsules could have all planes rejected above.
			// If that happens, select the most opposing plane including those that
			// may point the same direction as N. 
			if (BestPlaneIndex == INDEX_NONE)
			{
				// This always returns a valid plane.
				BestPlaneIndex = ScaledConvex.GetMostOpposingPlane(N);
			}

			CheckPlaneIndex(BestPlaneIndex, ScaledConvex, X, N, InMaxDistance, VertexIndex);
			return BestPlaneIndex;
		}

		// GJK was failing to generate vertex indices. This issue was tracked down to a degenerate simplex in GJK, but we may need this again one day...
		template<typename ConvexImplicitType>
		bool CheckVertexIndex(const ConvexImplicitType& Convex, const int32 VertexIndex)
		{
			CHAOS_COLLISIONERROR_CLOG(VertexIndex == INDEX_NONE, TEXT("GJKContactPointMargin invalid vertex index %d %s"), VertexIndex, *Convex.ToString());
			CHAOS_COLLISIONERROR_ENSURE(VertexIndex != INDEX_NONE);
			return VertexIndex != INDEX_NONE;
		}

		template <typename ConvexImplicitType1, typename ConvexImplicitType2>
		void ConstructConvexConvexOneShotManifold(
			const ConvexImplicitType1& Convex1,
			const FRigidTransform3& Convex1Transform, //world
			const ConvexImplicitType2& Convex2,
			const FRigidTransform3& Convex2Transform, //world
			const FReal Dt,
			FPBDCollisionConstraint& Constraint)
		{
			SCOPE_CYCLE_COUNTER_MANIFOLD();

			const uint32 SpaceDimension = 3;
			const bool bConvex1IsCapsule = (Convex1.GetType() & ~(ImplicitObjectType::IsInstanced | ImplicitObjectType::IsScaled)) == ImplicitObjectType::Capsule;

			// We only build one shot manifolds once
			// All convexes are pre-scaled, or wrapped in TImplicitObjectScaled
			//ensure(Constraint.GetManifoldPoints().Num() == 0);
			ensure(Convex1Transform.GetScale3D() == FVec3(1.0f, 1.0f, 1.0f));
			ensure(Convex2Transform.GetScale3D() == FVec3(1.0f, 1.0f, 1.0f));

			// Get the adjusted margins for each convex
			const FReal Margin1 = Constraint.GetCollisionMargin0();
			const FReal Margin2 = Constraint.GetCollisionMargin1();
			const FRigidTransform3 Convex2ToConvex1Transform = Convex2Transform.GetRelativeTransformNoScale(Convex1Transform);

			if (!bChaos_Manifold_EnableGjkWarmStart)
			{
				Constraint.GetGJKWarmStartData().Reset();
			}

			// Find the deepest penetration. This is used to determine the planes and points to use for the manifold
			// MaxMarginDelta is an upper bound on the distance from the contact on the rounded core shape to the actual shape surface. 
			FReal MaxMarginDelta = FReal(0);
			int32 VertexIndexA = INDEX_NONE, VertexIndexB = INDEX_NONE;
			FContactPoint GJKContactPoint = GJKContactPointMargin(Convex1, Convex2, Convex2ToConvex1Transform, Margin1, Margin2, Constraint.GetGJKWarmStartData(), MaxMarginDelta, VertexIndexA, VertexIndexB);
			PHYSICS_CSV_CUSTOM_EXPENSIVE(PhysicsCounters, NumManifoldsGJKCalled, 1, ECsvCustomStatOp::Accumulate);

			// We are seeing very rare cases where the VertexIndex is not set from an FConvex, even though the convex has vertices
			// @todo(chaos): track down this problem - there is no path through GJKContactPointMargin (for FConvex) which does not set the VertexIndex
			if (!CheckVertexIndex(Convex1, VertexIndexA) || !CheckVertexIndex(Convex2, VertexIndexB))
			{
				return;
			}

			const bool bCanUpdateManifold = bChaos_Collision_EnableManifoldGJKReplace;
			if (bCanUpdateManifold && Constraint.TryAddManifoldContact(GJKContactPoint))
			{
				PHYSICS_CSV_CUSTOM_EXPENSIVE(PhysicsCounters, NumManifoldsMaintained, 1, ECsvCustomStatOp::Accumulate);
				return;
			}

			Constraint.ResetActiveManifoldContacts();

			// GJK is using margins and rounded corner, so if we have a corner-to-corner contact it will under-report the actual distance by an amount that depends on how
			// "pointy" the edge/corner is - this error is bounded by MaxMarginDelta.
			const FReal GJKCullDistance = Constraint.GetCullDistance() + MaxMarginDelta;
			if (GJKContactPoint.Phi > GJKCullDistance)
			{
				PHYSICS_CSV_CUSTOM_EXPENSIVE(PhysicsCounters, NumManifoldsGJKCulled, 1, ECsvCustomStatOp::Accumulate);
				return;
			}

			PHYSICS_CSV_CUSTOM_EXPENSIVE(PhysicsCounters, NumManifoldsCreated, 1, ECsvCustomStatOp::Accumulate);


			// @todo(chaos): get the vertex index from GJK and use to to get the plane
			const FVec3 SeparationDirectionLocalConvex1 = Convex2ToConvex1Transform.TransformVectorNoScale(GJKContactPoint.ShapeContactNormal);
			const int32 MostOpposingPlaneIndexConvex1 = SelectContactPlane(Convex1, GJKContactPoint.ShapeContactPoints[0], SeparationDirectionLocalConvex1, Margin1, VertexIndexA);
			if (MostOpposingPlaneIndexConvex1 == INDEX_NONE)
			{
				return;
			}
			FVec3 BestPlanePosition1, BestPlaneNormal1;
			Convex1.GetPlaneNX(MostOpposingPlaneIndexConvex1, BestPlaneNormal1, BestPlanePosition1);
			const FReal BestPlaneDotNormalConvex1 = !bConvex1IsCapsule ? FMath::Abs(FVec3::DotProduct(-SeparationDirectionLocalConvex1, BestPlaneNormal1)) : -FLT_MAX;

			// Now for Convex2
			const FVec3 SeparationDirectionLocalConvex2 = GJKContactPoint.ShapeContactNormal;
			const int32 MostOpposingPlaneIndexConvex2 = SelectContactPlane(Convex2, GJKContactPoint.ShapeContactPoints[1], -SeparationDirectionLocalConvex2, Margin2, VertexIndexB);
			if (MostOpposingPlaneIndexConvex2 == INDEX_NONE)
			{
				return;
			}
			FVec3 BestPlanePosition2, BestPlaneNormal2;
			Convex2.GetPlaneNX(MostOpposingPlaneIndexConvex2, BestPlaneNormal2, BestPlanePosition2);
			const FReal BestPlaneDotNormalConvex2 = FMath::Abs(FVec3::DotProduct(SeparationDirectionLocalConvex2, BestPlaneNormal2));

			const FReal SmallBiasToPreventFeatureFlipping = 0.002f; // This improves frame coherence by penalizing convex 1 in favour of convex 2
			bool ReferenceFaceConvex1 = true; // Is the reference face on convex1 or convex2?
			if (BestPlaneDotNormalConvex2 + SmallBiasToPreventFeatureFlipping > BestPlaneDotNormalConvex1)
			{
				ReferenceFaceConvex1 = false;
			}

			// Is this a vertex-plane or edge-edge contact? 
			const FReal PlaneContactNormalEpsilon = Chaos_Collision_Manifold_PlaneContactNormalEpsilon;
			const bool bIsPlaneContact = FMath::IsNearlyEqual(BestPlaneDotNormalConvex1, (FReal)1., PlaneContactNormalEpsilon) || FMath::IsNearlyEqual(BestPlaneDotNormalConvex2, (FReal)1., PlaneContactNormalEpsilon);

			// For edge-edge contacts, we find the edges involved and project the contact onto the edges (if necessary - see below)
			if (!bIsPlaneContact)
			{
				if (ForceOneShotManifoldEdgeEdgeCaseZeroCullDistance && GJKContactPoint.Phi > 0)
				{
					return;
				}

				// This is an edge-edge contact
				GJKContactPoint.ContactType = EContactPointType::EdgeEdge;

				// @todo(chaos): remove this if we ditch convex margins
				// We only need to find the best edges when we have convex margins enabled because then the GJK result
				// is a contact on the rounded margin-reduced shape and not the actual shape. If we are using zero
				// margins, then the GJK result is already on the real shape surface. Likewise, if we have a
				// quadratic-versus-polygonal contact, the GJK results are already on the real surfaces.
				const FReal PolygonalMargin1 = Constraint.IsQuadratic0() ? FReal(0) : Margin1;
				const FReal PolygonalMargin2 = Constraint.IsQuadratic1() ? FReal(0) : Margin2;
				if ((PolygonalMargin1 > 0) || (PolygonalMargin2 > 0))
				{
					// @todo(chaos): this does not work well when the edges are parallel. We should always have points with zero
					// position delta perpendicular to the normal, but that is not the case for parallel edges
					FVec3 ShapeEdgePos1 = Convex1.GetClosestEdgePosition(MostOpposingPlaneIndexConvex1, GJKContactPoint.ShapeContactPoints[0]);
					FVec3 ShapeEdgePos2 = Convex2.GetClosestEdgePosition(MostOpposingPlaneIndexConvex2, GJKContactPoint.ShapeContactPoints[1]);
					if (bConvex1IsCapsule)
					{
						ShapeEdgePos1 -= Margin1 * SeparationDirectionLocalConvex1;
					}

					const FVec3 EdgePos1In2 = Convex2ToConvex1Transform.InverseTransformPositionNoScale(ShapeEdgePos1);
					const FVec3 EdgePos2In2 = ShapeEdgePos2;
					const FReal EdgePhi = FVec3::DotProduct(EdgePos1In2 - EdgePos2In2, GJKContactPoint.ShapeContactNormal);

					// We now have an accurate separation, so reject points beyond cull distance.
					// This is quite important because if we use a larger cull distance for edge-edge contacts than for vertex-plane
					// contacts, the pruning of unnecessary edge-edge contacts does not work (Tri Mesh and heighfield)
					if (EdgePhi > Constraint.GetCullDistance())
					{
						return;
					}

					// @todo(chaos): we leave the contact point on the second shape because this is better for triangle meshes (see PruneContactEdges). For other shapes it should probably be the average
					//const FVec3 EdgePosIn2 = FReal(0.5) * (EdgePos1In2 + EdgePos2In2);
					//GJKContactPoint.ShapeContactPoints[0] = Convex2ToConvex1Transform.TransformPositionNoScale(EdgePosIn2 + FReal(0.5) * EdgePhi * GJKContactPoint.ShapeContactNormal);
					//GJKContactPoint.ShapeContactPoints[1] = EdgePosIn2 - FReal(0.5) * EdgePhi * GJKContactPoint.ShapeContactNormal;
					GJKContactPoint.ShapeContactPoints[0] = Convex2ToConvex1Transform.TransformPositionNoScale(EdgePos2In2 + EdgePhi * GJKContactPoint.ShapeContactNormal);
					GJKContactPoint.ShapeContactPoints[1] = EdgePos2In2;
					GJKContactPoint.Phi = EdgePhi;
					// Normal unchanged from GJK result
				}

				Constraint.AddOneshotManifoldContact(GJKContactPoint);
				return;
			}

			// For vertex-plane contacts, we use a convex face as the manifold plane
			const FVec3 RefSeparationDirection = ReferenceFaceConvex1 ? SeparationDirectionLocalConvex1 : SeparationDirectionLocalConvex2;
			const FVec3 RefPlaneNormal = ReferenceFaceConvex1 ? BestPlaneNormal1 : BestPlaneNormal2;
			const FVec3 RefPlanePosition = ReferenceFaceConvex1 ? BestPlanePosition1 : BestPlanePosition2;

			// @todo(chaos): fix use of hard-coded max array size
			// We will use a double buffer as an optimization
			const uint32 MaxContactPointCount = 32; // This should be tuned
			uint32 ContactPointCount = 0;
			FVec3* ClippedVertices = nullptr;
			const FRigidTransform3* RefConvexTM;
			FRigidTransform3 ConvexOtherToRef;
			FVec3 ClippedVertices1[MaxContactPointCount];
			FVec3 ClippedVertices2[MaxContactPointCount];

			{
				if (ReferenceFaceConvex1)
				{
					RefConvexTM = &Convex1Transform;
					ConvexOtherToRef = Convex2ToConvex1Transform;

					ClippedVertices = GenerateConvexManifoldClippedVertices(
						Convex1,
						Convex2,
						ConvexOtherToRef, 
						MostOpposingPlaneIndexConvex1,
						MostOpposingPlaneIndexConvex2,
						RefPlaneNormal,
						ClippedVertices1,
						ClippedVertices2,
						ContactPointCount,
						MaxContactPointCount);
				}
				else
				{
					RefConvexTM = &Convex2Transform;
					ConvexOtherToRef = Convex1Transform.GetRelativeTransformNoScale(Convex2Transform);

					ClippedVertices = GenerateConvexManifoldClippedVertices(
						Convex2,
						Convex1,
						ConvexOtherToRef,
						MostOpposingPlaneIndexConvex2,
						MostOpposingPlaneIndexConvex1,
						RefPlaneNormal,
						ClippedVertices1,
						ClippedVertices2,
						ContactPointCount,
						MaxContactPointCount);
				}
			}

			// If we have the max number of contact points already, they will be in cyclic order. Stability is better
			// if we solve points non-sequentially (e.g., on a box, solve one point, then it's opposite corner).
			// If we have more than 4 contacts, the contact reduction step already effectively does something similar.
			if (ContactPointCount == 4)
			{
				Swap(ClippedVertices[1], ClippedVertices[2]);
			}

			// Reduce number of contacts to the maximum allowed
			if (ContactPointCount > 4)
			{
				FRotation3 RotateSeperationToZ = FRotation3::FromRotatedVector(RefPlaneNormal, FVec3(0.0f, 0.0f, 1.0f));
				for (uint32 ContactPointIndex = 0; ContactPointIndex < ContactPointCount; ++ContactPointIndex)
				{
					ClippedVertices[ContactPointIndex] = RotateSeperationToZ * ClippedVertices[ContactPointIndex];
				}

				ContactPointCount = ReduceManifoldContactPoints(ClippedVertices, ContactPointCount);

				for (uint32 ContactPointIndex = 0; ContactPointIndex < ContactPointCount; ++ContactPointIndex)
				{
					ClippedVertices[ContactPointIndex] = RotateSeperationToZ.Inverse() * ClippedVertices[ContactPointIndex];
				}
			}

			// This is a vertex-plane contact
			EContactPointType ContactType = ReferenceFaceConvex1 ? EContactPointType::PlaneVertex : EContactPointType::VertexPlane;

			// Generate the contact points from the clipped vertices
			{
				for (uint32 ContactPointIndex = 0; ContactPointIndex < ContactPointCount; ++ContactPointIndex)
				{
					FContactPoint ContactPoint;
					FVec3 VertexInReferenceCoordinates = ClippedVertices[ContactPointIndex];
					if (bConvex1IsCapsule)
					{
						VertexInReferenceCoordinates -= Margin1 * RefSeparationDirection;
					}
					FVec3 PointProjectedOntoReferenceFace = VertexInReferenceCoordinates - FVec3::DotProduct(VertexInReferenceCoordinates - RefPlanePosition, RefPlaneNormal) * RefPlaneNormal;
					FVec3 ClippedPointInOtherCoordinates = ConvexOtherToRef.InverseTransformPositionNoScale(VertexInReferenceCoordinates);

					ContactPoint.ShapeContactPoints[0] = ReferenceFaceConvex1 ? PointProjectedOntoReferenceFace : ClippedPointInOtherCoordinates;
					ContactPoint.ShapeContactPoints[1] = ReferenceFaceConvex1 ? ClippedPointInOtherCoordinates : PointProjectedOntoReferenceFace;
					ContactPoint.ShapeContactNormal = SeparationDirectionLocalConvex2;
					ContactPoint.Phi = FVec3::DotProduct(PointProjectedOntoReferenceFace - VertexInReferenceCoordinates, ReferenceFaceConvex1 ? SeparationDirectionLocalConvex1 : -SeparationDirectionLocalConvex2);				
					ContactPoint.ContactType = ContactType;

					Constraint.AddOneshotManifoldContact(ContactPoint);
				}
			}
		}


		template <typename ConvexType>
		void ConstructCapsuleConvexOneShotManifold(
			const FImplicitCapsule3& Capsule, 
			const FRigidTransform3& CapsuleTransform,
			const ConvexType& Convex,
			const FRigidTransform3& ConvexTransform,
			const FReal CullDistance,
			FContactPointManifold& OutContactPoints)
		{
			SCOPE_CYCLE_COUNTER_MANIFOLD();
			check(OutContactPoints.Num() == 0);
			using FLineSegment3 = TSegment<FReal>;

			// Transform Capsule into Convex space
			// We only keep the segment at this point - we'll add the radius back later if we create contacts
			const FRigidTransform3 CapsuleToConvexTransform = CapsuleTransform.GetRelativeTransformNoScale(ConvexTransform);
			const FReal CapsuleRadius = Capsule.GetRadiusf();
			const FReal CapsuleHeight = Capsule.GetHeightf();
			const FLineSegment3 CapsuleAxisSegment = FLineSegment3(
				CapsuleToConvexTransform.TransformPositionNoScale(FVec3(Capsule.GetX1f())),
				CapsuleToConvexTransform.TransformVectorNoScale(FVec3(Capsule.GetAxisf())),
				CapsuleHeight);

			// No margins for the convex or the capsule for this GJK. We'll correct for the radius later...
			// NOTE: We are passing the convex first into GJK, but the output should put the capsule first
			FReal UnusedMaxMarginDelta = FReal(0);
			int32 VertexIndexA = INDEX_NONE, VertexIndexB = INDEX_NONE;
			const TGJKShape<FLineSegment3> GJKSegment(CapsuleAxisSegment);
			const TGJKShape<ConvexType> GJKConvex(Convex);
			FContactPoint GJKContactPoint = GJKContactPointSameSpace(GJKConvex, GJKSegment, UnusedMaxMarginDelta, VertexIndexA, VertexIndexB);

			// We are seeing very rare cases where the VertexIndex is not set from an FConvex, even though the convex has vertices
			// @todo(chaos): track down this problem - there is no path through GJKContactPointMargin (for FConvex) which does not set the VertexIndex
			if (!CheckVertexIndex(Convex, VertexIndexA))
			{
				return;
			}

			// Map contact back onto capsule
			GJKContactPoint.ShapeContactPoints[1] = GJKContactPoint.ShapeContactPoints[1] + CapsuleRadius * GJKContactPoint.ShapeContactNormal;
			GJKContactPoint.Phi = GJKContactPoint.Phi - CapsuleRadius;

			PHYSICS_CSV_CUSTOM_EXPENSIVE(PhysicsCounters, NumManifoldsGJKCalled, 1, ECsvCustomStatOp::Accumulate);

			if (GJKContactPoint.Phi > CullDistance)
			{
				PHYSICS_CSV_CUSTOM_EXPENSIVE(PhysicsCounters, NumManifoldsGJKCulled, 1, ECsvCustomStatOp::Accumulate);
				return;
			}

			PHYSICS_CSV_CUSTOM_EXPENSIVE(PhysicsCounters, NumManifoldsCreated, 1, ECsvCustomStatOp::Accumulate);

			// Is this a cylinder or end cap contact?
			const FReal CapsuleEndCapAxisNormalThreshold = FReal(0.01);	// About sin(0.5 deg)
			const FReal CapsuleAxisDotNormal = FVec3::DotProduct(CapsuleAxisSegment.GetAxis(), GJKContactPoint.ShapeContactNormal);
			const bool bIsCapsuleCylinderContact = (FMath::Abs(CapsuleAxisDotNormal) < CapsuleEndCapAxisNormalThreshold);

			// Find the best plane on the convex (most opposing direction)
			const FReal PlaneSearchDistance = FReal(0);	// use the miniumum (see SelectContactPlane)
			const int32 ConvexPlaneIndex = SelectContactPlane(Convex, GJKContactPoint.ShapeContactPoints[0], GJKContactPoint.ShapeContactNormal, PlaneSearchDistance, VertexIndexA);
			if (ConvexPlaneIndex == INDEX_NONE)
			{
				return;
			}
			FVec3 ConvexPlanePosition, ConvexPlaneNormal;
			Convex.GetPlaneNX(ConvexPlaneIndex, ConvexPlaneNormal, ConvexPlanePosition);
			const FReal ConvexPlaneNormalDotContactNormal = FVec3::DotProduct(ConvexPlaneNormal, GJKContactPoint.ShapeContactNormal);
			const FReal PlaneContactNormalEpsilon = Chaos_Collision_Manifold_PlaneContactNormalEpsilon;
			bool bIsConvexPlaneContact = FMath::IsNearlyEqual(ConvexPlaneNormalDotContactNormal, FReal(-1), PlaneContactNormalEpsilon);

			if (bIsCapsuleCylinderContact)
			{
				// Cylinder plane most opposing GJK result direction
				const FVec3 CapsulePlaneNormal = -(GJKContactPoint.ShapeContactNormal - CapsuleAxisDotNormal * CapsuleAxisSegment.GetAxis()).GetSafeNormal();

				// If we have a cylinder contact we generate 2 contact points from the clipped line segment
				// NOTE: There is an edge case where GJK returns a cylinder normal but both segment positions are
				// outside the box. This happens if you drop a capsule onto a box so that the segment exactly touches
				// a box face. It also happens if we mis-identify the contact as a cylinder contact because of
				// the normal check tolerance. Either way, we fall through to just using the GJK result
				const FReal PlaneTolerance = FReal(1e-2);
				const int32 MaxContactPointCount = 2;
				int32 ContactPointCount = 0;
				FVec3 ClippedVertices[MaxContactPointCount];
				GenerateLineConvexManifoldClippedVerticesSameSpace(
					Convex,
					CapsuleAxisSegment,
					ConvexPlaneIndex,
					CapsulePlaneNormal,
					ClippedVertices,
					ContactPointCount,
					MaxContactPointCount,
					PlaneTolerance);

				if (ContactPointCount > 0)
				{
					// Divisor used below to project the capsule contact onto the convex plane
					// See FMath::RayPlaneIntersection for math, but it is separated here to avoid excessive dot products
					const FReal ConvexDistanceDivisor = FVec3::DotProduct(CapsulePlaneNormal, ConvexPlaneNormal);
					if (FMath::Abs(ConvexDistanceDivisor) > UE_KINDA_SMALL_NUMBER)
					{
						const FReal ConvexDistanceMultiplier = FReal(1) / ConvexDistanceDivisor;

						for (int32 ContactPointIndex = 0; ContactPointIndex < ContactPointCount; ++ContactPointIndex)
						{
							// Capsule points are on cylinder surface, so easy to calculate
							const FVec3 CapsuleContactPoint = ClippedVertices[ContactPointIndex] - CapsuleRadius * CapsulePlaneNormal;

							// Project the capsule contacts onto the convex plane
							const FReal ConvexContactPointDistance = FVec3::DotProduct((ConvexPlanePosition - CapsuleContactPoint), ConvexPlaneNormal) * ConvexDistanceMultiplier;
							const FVec3 ConvexContactPoint = CapsuleContactPoint + ConvexContactPointDistance * CapsulePlaneNormal;

							FContactPoint& ContactPoint = OutContactPoints[OutContactPoints.AddUninitialized()];
							ContactPoint.ShapeContactPoints[0] = CapsuleToConvexTransform.InverseTransformPositionNoScale(CapsuleContactPoint);
							ContactPoint.ShapeContactPoints[1] = ConvexContactPoint;
							ContactPoint.ShapeContactNormal = CapsulePlaneNormal;
							ContactPoint.Phi = FVec3::DotProduct(CapsuleContactPoint - ConvexContactPoint, CapsulePlaneNormal);
							ContactPoint.FaceIndex = INDEX_NONE;
							ContactPoint.ContactType = EContactPointType::PlaneVertex;
						}

						return;
					}
				}
			}
			else if (bIsConvexPlaneContact)
			{
				// If we get here we are creating a convex face versus end cap collision
				// The end cap collision gets added at the end (the GJK result), but we also add the clipped capsule cylinder points as well
				// but only if the cylinder is not standing vertically on the face
				const FReal CapsuleCylinderTolerance = FReal(0.707);	// about 45 deg
				const FReal CapsuleAxisDotConvexNormal = FVec3::DotProduct(ConvexPlaneNormal, CapsuleAxisSegment.GetAxis());
				if (FMath::Abs(CapsuleAxisDotConvexNormal) < CapsuleCylinderTolerance)
				{
					// The line segment on the surface of the cylinder closest to the contact
					const FVec3 CapsuleCylinderNormal = (ConvexPlaneNormal - CapsuleAxisDotConvexNormal * CapsuleAxisSegment.GetAxis()).GetUnsafeNormal();
					const FLineSegment3 CapsuleCylinderSegment = FLineSegment3(CapsuleAxisSegment.GetX1() - CapsuleRadius * CapsuleCylinderNormal, CapsuleAxisSegment.GetAxis(), CapsuleHeight);

					// Clip the cylinder segment to the convex face
					const FReal PlaneTolerance = FReal(1e-2);
					const int32 MaxContactPointCount = 2;
					int32 ContactPointCount = 0;
					FVec3 ClippedVertices[MaxContactPointCount];
					GenerateLineConvexManifoldClippedVerticesSameSpace(
						Convex,
						CapsuleCylinderSegment,
						ConvexPlaneIndex,
						ConvexPlaneNormal,
						ClippedVertices,
						ContactPointCount,
						MaxContactPointCount,
						PlaneTolerance);

					const FReal PointDistanceToleranceFraction = Chaos_Collision_Manifold_CapsuleMinContactDistanceFraction;
					const FReal PointDistanceToleranceSq = FMath::Square(PointDistanceToleranceFraction * CapsuleRadius);

					// Add the vertices if not clipped away
					for (int32 ContactPointIndex = 0; ContactPointIndex < ContactPointCount; ++ContactPointIndex)
					{
						// Don't add the point if it's very close to the GJK result because we're adding that too below
						const FVec3 CapsuleContactPoint = ClippedVertices[ContactPointIndex];
						const FReal CapsuleContactGJKDistanceSq = (CapsuleContactPoint - GJKContactPoint.ShapeContactPoints[1]).SizeSquared();
						if (CapsuleContactGJKDistanceSq > PointDistanceToleranceSq)
						{
							const FVec3 ConvexContactPoint = CapsuleContactPoint - FVec3::DotProduct(CapsuleContactPoint - ConvexPlanePosition, ConvexPlaneNormal) * ConvexPlaneNormal;

							FContactPoint& ContactPoint = OutContactPoints[OutContactPoints.AddUninitialized()];
							ContactPoint.ShapeContactPoints[0] = CapsuleToConvexTransform.InverseTransformPositionNoScale(CapsuleContactPoint);
							ContactPoint.ShapeContactPoints[1] = ConvexContactPoint;
							ContactPoint.ShapeContactNormal = ConvexPlaneNormal;
							ContactPoint.Phi = FVec3::DotProduct(CapsuleContactPoint - ConvexContactPoint, ConvexPlaneNormal);
							ContactPoint.FaceIndex = INDEX_NONE;
							ContactPoint.ContactType = EContactPointType::VertexPlane;
						}
					}
				}

				// @todo(chaos): handle the case where we have a convex on the end cap by adding a couple stabilizing points around the main contact
			}
		
			// We have an end-cap collision, so add the GJK point
			// NOTE: We are not using convex margins, so the GJK result is already on both shape surfaces
			// NOTE: we must swap the shape orders and transform the capsule data back into its space
			{
				FContactPoint ContactPoint;
				ContactPoint.ShapeContactPoints[0] = CapsuleToConvexTransform.InverseTransformPositionNoScale(GJKContactPoint.ShapeContactPoints[1]);
				ContactPoint.ShapeContactPoints[1] = GJKContactPoint.ShapeContactPoints[0];
				ContactPoint.ShapeContactNormal = -GJKContactPoint.ShapeContactNormal;
				ContactPoint.Phi = GJKContactPoint.Phi;
				ContactPoint.FaceIndex = INDEX_NONE;
				ContactPoint.ContactType = bIsConvexPlaneContact ? EContactPointType::VertexPlane : EContactPointType::EdgeEdge;
				OutContactPoints.Add(ContactPoint);
				return;
			}
		}

		void ConstructCapsuleTriangleOneShotManifold(const FImplicitCapsule3& Capsule, const FTriangle& Triangle, const FReal CullDistance, FContactPointManifold& OutContactPoints)
		{
			// @todo(chaos): make custom capsule-triangle manifold function.
			// NOTE: ConstructCapsuleConvexOneShotManifold could be used but it has issues when we collide a triangle edge with the capsule cylinder - that needs fixing...
			FPBDCollisionConstraint Constraint = FPBDCollisionConstraint::MakeTriangle(&Capsule);
			ConstructConvexConvexOneShotManifold(Capsule, FRigidTransform3::Identity, Triangle, FRigidTransform3::Identity, 0.0f, Constraint);
			for (const FManifoldPoint& ManifoldPoint : Constraint.GetManifoldPoints())
			{
				OutContactPoints.Add(ManifoldPoint.ContactPoint);
			}
			return;
		}

		// @todo(chaos): don't use GJK/EPA for triangle-convex so we can avoid using margins and errors related to misidentifying edge contacts
		template <typename ConvexType>
		void ConstructPlanarConvexTriangleOneShotManifold(const ConvexType& Convex, const FTriangle& Triangle, const FReal CullDistance, FContactPointManifold& OutContactPoints)
		{
			SCOPE_CYCLE_COUNTER_MANIFOLD();
			check(OutContactPoints.Num() == 0);

			// Find the deepest penetration. This is used to determine the planes and points to use for the manifold
			// MaxMarginDelta is an upper bound on the distance from the contact on the rounded core shape to the actual shape surface. 
			const FReal MarginMultiplier = Chaos_Collision_Manifold_TriangleConvexMarginMultiplier;
			const FReal ConvexMargin = MarginMultiplier * FReal(Convex.GetMarginf());
			FReal MaxMarginDelta = FReal(0);
			int32 VertexIndexA = INDEX_NONE, VertexIndexB = INDEX_NONE;
			const TGJKCoreShape<ConvexType> GJKConvex(Convex, ConvexMargin);
			const TGJKShape<FTriangle> GJKTriangle(Triangle);
			const FVec3 InitialGJKDir = -Triangle.GetCentroid();
			FContactPoint GJKContactPoint = GJKContactPointSameSpace(GJKConvex, GJKTriangle, MaxMarginDelta, VertexIndexA, VertexIndexB, InitialGJKDir);

			// We are seeing very rare cases where the VertexIndex is not set from an FConvex, even though the convex has vertices
			// @todo(chaos): track down this problem - there is no path through GJKContactPointMargin (for FConvex) which does not set the VertexIndex
			if (!CheckVertexIndex(Convex, VertexIndexA))
			{
				return;
			}

			PHYSICS_CSV_CUSTOM_EXPENSIVE(PhysicsCounters, NumManifoldsGJKCalled, 1, ECsvCustomStatOp::Accumulate);

			// GJK is using margins and rounded corner, so if we have a corner-to-corner contact it will under-report the actual distance by an amount that depends on how
			// "pointy" the edge/corner is - this error is bounded by MaxMarginDelta.
			const FReal GJKCullDistance = CullDistance + MaxMarginDelta;
			if (GJKContactPoint.Phi > GJKCullDistance)
			{
				PHYSICS_CSV_CUSTOM_EXPENSIVE(PhysicsCounters, NumManifoldsGJKCulled, 1, ECsvCustomStatOp::Accumulate);
				return;
			}

			PHYSICS_CSV_CUSTOM_EXPENSIVE(PhysicsCounters, NumManifoldsCreated, 1, ECsvCustomStatOp::Accumulate);

			// See if the triangle face is a good choice for the manifold plane. We preferentially use the triangle face with
			// a high threshold to avoid spurious edge/vertex collisions.
			const FVec3 TrianglePlaneNormal = Triangle.GetNormal();
			const FVec3 TrianglePlanePosition = Triangle[0];
			const FReal TrianglePlaneNormalDotContactNormal = FVec3::DotProduct(TrianglePlaneNormal, GJKContactPoint.ShapeContactNormal);

			// Find the best opposing plane on the convex
			// @todo(chaos): handle zero margins...
			const FReal PlaneSearchDistance = ConvexMargin;
			const int32 ConvexPlaneIndex = SelectContactPlane(Convex, GJKContactPoint.ShapeContactPoints[0], GJKContactPoint.ShapeContactNormal, PlaneSearchDistance, VertexIndexA);
			if (ConvexPlaneIndex == INDEX_NONE)
			{
				return;
			}
			FVec3 ConvexPlanePosition, ConvexPlaneNormal;
			Convex.GetPlaneNX(ConvexPlaneIndex, ConvexPlaneNormal, ConvexPlanePosition);
			const FReal ConvexPlaneNormalDotContactNormal = FVec3::DotProduct(ConvexPlaneNormal, GJKContactPoint.ShapeContactNormal);

			// Is either plane a good choice, or do we have and edge-edge collision
			// For edge-edge contacts, we find the edges involved and project the contact onto the edges (if necessary - see below)
			const FReal PlaneContactNormalEpsilon = Chaos_Collision_Manifold_PlaneContactNormalEpsilon;
			bool bIsPlaneContact = FMath::IsNearlyEqual(ConvexPlaneNormalDotContactNormal, FReal(-1), PlaneContactNormalEpsilon) || FMath::IsNearlyEqual(TrianglePlaneNormalDotContactNormal, FReal(1), PlaneContactNormalEpsilon);
			if (!bIsPlaneContact)
			{
				if (ForceOneShotManifoldEdgeEdgeCaseZeroCullDistance && GJKContactPoint.Phi > 0)
				{
					return;
				}

				// @todo(chaos): remove this if we ditch convex margins
				// We only need to find the best edges when we have convex margins enabled because then the GJK result
				// is a contact on the rounded margin-reduced shape and not the actual shape. If we are using zero
				// margins, then the GJK result is already on the real shape surface. Likewise, if we have a
				// quadratic-versus-polygonal contact, the GJK results are already on the real surfaces.
				if (ConvexMargin > 0)
				{
					FVec3 ConvexEdgePos0, ConvexEdgePos1;
					const FVec3 ConvexEdgeNearPos = Convex.GetClosestEdge(ConvexPlaneIndex, GJKContactPoint.ShapeContactPoints[0], ConvexEdgePos0, ConvexEdgePos1);

					FVec3 TriangleEdgePos0, TriangleEdgePos1;
					const FVec3 TriangleEdgeNearPos = Triangle.GetClosestEdge(0, GJKContactPoint.ShapeContactPoints[1], TriangleEdgePos0, TriangleEdgePos1);

					// Does the normal we found from the edges match the GJK normal? If not, we have a false negative in the check for
					// bIsPlaneContact above caused by margins. If this happens, fall back to a plane contact.
					const FVec3 EdgeNormal = FVec3::CrossProduct(ConvexEdgePos1 - ConvexEdgePos0, TriangleEdgePos1 - TriangleEdgePos0).GetSafeNormal();
					const FReal EdgeNormalDotContactNormal = FVec3::DotProduct(EdgeNormal, GJKContactPoint.ShapeContactNormal);
					bIsPlaneContact = (FMath::Abs(EdgeNormalDotContactNormal) < Chaos_Collision_Manifold_EdgeContactNormalThreshold);

					if (!bIsPlaneContact)
					{
						const FReal EdgePhi = FVec3::DotProduct(ConvexEdgeNearPos - TriangleEdgeNearPos, GJKContactPoint.ShapeContactNormal);

						// We now have an accurate separation, so reject points beyond cull distance.
						// This is quite important because if we use a larger cull distance for edge-edge contacts than for vertex-plane
						// contacts, the pruning of unnecessary edge-edge contacts does not work (Tri Mesh and heighfield)
						if (EdgePhi > CullDistance)
						{
							return;
						}

						// Remove anmy lateral delta from the contact on point on the convex
						GJKContactPoint.ShapeContactPoints[0] = TriangleEdgeNearPos + EdgePhi * GJKContactPoint.ShapeContactNormal;
						GJKContactPoint.ShapeContactPoints[1] = TriangleEdgeNearPos;
						// Normal unchanged from GJK result
						GJKContactPoint.Phi = EdgePhi;
					}
				}

				if (!bIsPlaneContact)
				{
					GJKContactPoint.ContactType = EContactPointType::EdgeEdge;
					OutContactPoints.Add(GJKContactPoint);
					return;
				}
			}

			// We prefer to use the triangle face for the contact to avoid collisions with internal edges and vertices
			const EContactPointType ContactType = (TrianglePlaneNormalDotContactNormal > Chaos_Collision_Manifold_TriangleContactNormalThreshold) ? EContactPointType::VertexPlane : EContactPointType::PlaneVertex;

			// @todo(chaos): fix use of hard-coded max array size
			// We will use a double buffer as an optimization
			const int32 MaxContactPointCount = 32; // This should be tuned
			int32 ContactPointCount = 0;
			FVec3 RefPlaneNormal;
			FVec3* ClippedVertices = nullptr;
			FVec3 ClippedVertices1[MaxContactPointCount];
			FVec3 ClippedVertices2[MaxContactPointCount];

			// Generate the set of vertices by clipping the convex face against the triangle face in the selected face plane
			if (ContactType == EContactPointType::VertexPlane)
			{
				// We have a triangle face contact
				RefPlaneNormal = TrianglePlaneNormal;

				ClippedVertices = GenerateConvexManifoldClippedVertices(
					Triangle,
					Convex,
					FRigidTransform3(),
					0,
					ConvexPlaneIndex,
					TrianglePlaneNormal,
					ClippedVertices1,
					ClippedVertices2,
					(uint32&)ContactPointCount,
					(uint32)MaxContactPointCount);

			}
			else
			{
				// We have a convex face contact
				RefPlaneNormal = ConvexPlaneNormal;

				ClippedVertices = GenerateConvexManifoldClippedVertices(
					Convex,
					Triangle,
					FRigidTransform3(),
					ConvexPlaneIndex,
					0,
					ConvexPlaneNormal,
					ClippedVertices1,
					ClippedVertices2,
					(uint32&)ContactPointCount,
					(uint32)MaxContactPointCount);
			}

			// If we have the max number of contact points already, they will be in cyclic order. Stability is better
			// if we solve points non-sequentially (e.g., on a box, solve one point, then it's opposite corner).
			// If we have more than 4 contacts, the contact reduction step already effectively does something similar.
			if (ContactPointCount == 4)
			{
				Swap(ClippedVertices[1], ClippedVertices[2]);
			}

			// Reduce number of contacts to the maximum allowed
			if (ContactPointCount > 4)
			{
				FRotation3 RotateSeperationToZ = FRotation3::FromRotatedVector(RefPlaneNormal, FVec3(0.0f, 0.0f, 1.0f));
				for (int32 ContactPointIndex = 0; ContactPointIndex < ContactPointCount; ++ContactPointIndex)
				{
					ClippedVertices[ContactPointIndex] = RotateSeperationToZ * ClippedVertices[ContactPointIndex];
				}

				ContactPointCount = ReduceManifoldContactPoints(ClippedVertices, ContactPointCount);

				for (int32 ContactPointIndex = 0; ContactPointIndex < ContactPointCount; ++ContactPointIndex)
				{
					ClippedVertices[ContactPointIndex] = RotateSeperationToZ.Inverse() * ClippedVertices[ContactPointIndex];
				}
			}

			// Generate the contact points from the clipped vertices
			check(ContactPointCount + OutContactPoints.Num() <= OutContactPoints.Max());

			if (ContactType == EContactPointType::VertexPlane)
			{
				// Triangle face contact (clipped vertices are convex vertices)
				for (int32 ContactPointIndex = 0; ContactPointIndex < ContactPointCount; ++ContactPointIndex)
				{
					FContactPoint& ContactPoint = OutContactPoints[OutContactPoints.AddUninitialized()];
					const FVec3& ConvexContactPoint = ClippedVertices[ContactPointIndex];
					const FVec3 TriangleContactPoint = ConvexContactPoint - FVec3::DotProduct(ConvexContactPoint - TrianglePlanePosition, TrianglePlaneNormal) * TrianglePlaneNormal;

					ContactPoint.ShapeContactPoints[0] = ConvexContactPoint;
					ContactPoint.ShapeContactPoints[1] = TriangleContactPoint;
					ContactPoint.ShapeContactNormal = TrianglePlaneNormal;
					ContactPoint.Phi = FVec3::DotProduct(ConvexContactPoint - TriangleContactPoint, TrianglePlaneNormal);
					ContactPoint.ContactType = ContactType;
				}
			}
			else
			{
				// Convex face contact (clipped vertices are triangle vertices)
				for (int32 ContactPointIndex = 0; ContactPointIndex < ContactPointCount; ++ContactPointIndex)
				{
					FContactPoint& ContactPoint = OutContactPoints[OutContactPoints.AddUninitialized()];
					const FVec3& TriangleContactPoint = ClippedVertices[ContactPointIndex];
					const FVec3 ConvexContactPoint = TriangleContactPoint - FVec3::DotProduct(TriangleContactPoint - ConvexPlanePosition, ConvexPlaneNormal) * ConvexPlaneNormal;

					ContactPoint.ShapeContactPoints[0] = ConvexContactPoint;
					ContactPoint.ShapeContactPoints[1] = TriangleContactPoint;
					ContactPoint.ShapeContactNormal = -ConvexPlaneNormal;
					ContactPoint.Phi = FVec3::DotProduct(ConvexContactPoint - TriangleContactPoint, -ConvexPlaneNormal);
					ContactPoint.ContactType = ContactType;
				}
			}
		}


		//
		// Explicit instantiations of all convex-convex manifold combinations we support
		// Box, Convex, Scaled-Convex
		//

		template 
		void ConstructConvexConvexOneShotManifold<FImplicitBox3, FImplicitBox3>(
			const FImplicitBox3& Implicit1,
			const FRigidTransform3& Convex1Transform, //world
			const FImplicitBox3& Implicit2,
			const FRigidTransform3& Convex2Transform, //world
			const FReal Dt,
			FPBDCollisionConstraint& Constraint);

		template
		void ConstructConvexConvexOneShotManifold<FImplicitBox3, FImplicitConvex3>(
			const FImplicitBox3& Implicit1,
			const FRigidTransform3& Convex1Transform, //world
			const FImplicitConvex3& Implicit2,
			const FRigidTransform3& Convex2Transform, //world
			const FReal Dt,
			FPBDCollisionConstraint& Constraint);

		template
		void ConstructConvexConvexOneShotManifold<FImplicitConvex3, FImplicitBox3>(
			const FImplicitConvex3& Implicit1,
			const FRigidTransform3& Convex1Transform, //world
			const FImplicitBox3& Implicit2,
			const FRigidTransform3& Convex2Transform, //world
			const FReal Dt,
			FPBDCollisionConstraint& Constraint);

		template
		void ConstructConvexConvexOneShotManifold<FImplicitBox3, TImplicitObjectInstanced<FImplicitConvex3>>(
			const FImplicitBox3& Implicit1,
			const FRigidTransform3& Convex1Transform, //world
			const TImplicitObjectInstanced<FImplicitConvex3>& Implicit2,
			const FRigidTransform3& Convex2Transform, //world
			const FReal Dt,
			FPBDCollisionConstraint& Constraint);

		template
		void ConstructConvexConvexOneShotManifold<TImplicitObjectInstanced<FImplicitConvex3>, FImplicitBox3>(
			const TImplicitObjectInstanced<FImplicitConvex3>& Implicit1,
			const FRigidTransform3& Convex1Transform, //world
			const FImplicitBox3& Implicit2,
			const FRigidTransform3& Convex2Transform, //world
			const FReal Dt,
			FPBDCollisionConstraint& Constraint);

		template
		void ConstructConvexConvexOneShotManifold<FImplicitBox3, TImplicitObjectScaled<FImplicitConvex3>>(
			const FImplicitBox3& Implicit1,
			const FRigidTransform3& Convex1Transform, //world
			const TImplicitObjectScaled<FImplicitConvex3>& Implicit2,
			const FRigidTransform3& Convex2Transform, //world
			const FReal Dt,
			FPBDCollisionConstraint& Constraint);

		template
		void ConstructConvexConvexOneShotManifold<TImplicitObjectScaled<FImplicitConvex3>, FImplicitBox3>(
			const TImplicitObjectScaled<FImplicitConvex3>& Implicit1,
			const FRigidTransform3& Convex1Transform, //world
			const FImplicitBox3& Implicit2,
			const FRigidTransform3& Convex2Transform, //world
			const FReal Dt,
			FPBDCollisionConstraint& Constraint);

		template
		void ConstructConvexConvexOneShotManifold<FImplicitConvex3, FImplicitConvex3>(
			const FImplicitConvex3& Implicit1,
			const FRigidTransform3& Convex1Transform, //world
			const FImplicitConvex3& Implicit2,
			const FRigidTransform3& Convex2Transform, //world
			const FReal Dt,
			FPBDCollisionConstraint& Constraint);

		template
		void ConstructConvexConvexOneShotManifold<TImplicitObjectInstanced<FImplicitConvex3>, FImplicitConvex3>(
			const TImplicitObjectInstanced<FImplicitConvex3>& Implicit1,
			const FRigidTransform3& Convex1Transform, //world
			const FImplicitConvex3& Implicit2,
			const FRigidTransform3& Convex2Transform, //world
			const FReal Dt,
			FPBDCollisionConstraint& Constraint);

		template
		void ConstructConvexConvexOneShotManifold<TImplicitObjectScaled<FImplicitConvex3>, FImplicitConvex3>(
			const TImplicitObjectScaled<FImplicitConvex3>& Implicit1,
			const FRigidTransform3& Convex1Transform, //world
			const FImplicitConvex3& Implicit2,
			const FRigidTransform3& Convex2Transform, //world
			const FReal Dt,
			FPBDCollisionConstraint& Constraint);

		template
		void ConstructConvexConvexOneShotManifold<FImplicitConvex3, TImplicitObjectInstanced<FImplicitConvex3>>(
			const FImplicitConvex3& Implicit1,
			const FRigidTransform3& Convex1Transform, //world
			const TImplicitObjectInstanced<FImplicitConvex3>& Implicit2,
			const FRigidTransform3& Convex2Transform, //world
			const FReal Dt,
			FPBDCollisionConstraint& Constraint);

		template
		void ConstructConvexConvexOneShotManifold<FImplicitConvex3, TImplicitObjectScaled<FImplicitConvex3>>(
			const FImplicitConvex3& Implicit1,
			const FRigidTransform3& Convex1Transform, //world
			const TImplicitObjectScaled<FImplicitConvex3>& Implicit2,
			const FRigidTransform3& Convex2Transform, //world
			const FReal Dt,
			FPBDCollisionConstraint& Constraint);

		template
		void ConstructConvexConvexOneShotManifold<TImplicitObjectInstanced<FImplicitConvex3>, TImplicitObjectInstanced<FImplicitConvex3>>(
			const TImplicitObjectInstanced<FImplicitConvex3>& Implicit1,
			const FRigidTransform3& Convex1Transform, //world
			const TImplicitObjectInstanced<FImplicitConvex3>& Implicit2,
			const FRigidTransform3& Convex2Transform, //world
			const FReal Dt,
			FPBDCollisionConstraint& Constraint);

		template
		void ConstructConvexConvexOneShotManifold<TImplicitObjectScaled<FImplicitConvex3>, TImplicitObjectInstanced<FImplicitConvex3>>(
			const TImplicitObjectScaled<FImplicitConvex3>& Implicit1,
			const FRigidTransform3& Convex1Transform, //world
			const TImplicitObjectInstanced<FImplicitConvex3>& Implicit2,
			const FRigidTransform3& Convex2Transform, //world
			const FReal Dt,
			FPBDCollisionConstraint& Constraint);

		template
		void ConstructConvexConvexOneShotManifold<TImplicitObjectInstanced<FImplicitConvex3>, TImplicitObjectScaled<FImplicitConvex3>>(
			const TImplicitObjectInstanced<FImplicitConvex3>& Implicit1,
			const FRigidTransform3& Convex1Transform, //world
			const TImplicitObjectScaled<FImplicitConvex3>& Implicit2,
			const FRigidTransform3& Convex2Transform, //world
			const FReal Dt,
			FPBDCollisionConstraint& Constraint);

		template
		void ConstructConvexConvexOneShotManifold<TImplicitObjectScaled<FImplicitConvex3>, TImplicitObjectScaled<FImplicitConvex3>>(
			const TImplicitObjectScaled<FImplicitConvex3>& Implicit1,
			const FRigidTransform3& Convex1Transform, //world
			const TImplicitObjectScaled<FImplicitConvex3>& Implicit2,
			const FRigidTransform3& Convex2Transform, //world
			const FReal Dt,
			FPBDCollisionConstraint& Constraint);
	

		template
		void ConstructCapsuleConvexOneShotManifold(
			const FImplicitCapsule3& Capsule,
			const FRigidTransform3& CapsuleTransform,
			const FImplicitConvex3& Convex,
			const FRigidTransform3& ConvexTransform,
			const FReal CullDistance,
			FContactPointManifold& OutContactPoints);

		template
		void ConstructCapsuleConvexOneShotManifold(
			const FImplicitCapsule3& Capsule,
			const FRigidTransform3& CapsuleTransform,
			const TImplicitObjectInstanced<FImplicitConvex3>& Convex,
			const FRigidTransform3& ConvexTransform,
			const FReal CullDistance,
			FContactPointManifold& OutContactPoints);

		template
		void ConstructCapsuleConvexOneShotManifold(
			const FImplicitCapsule3& Capsule,
			const FRigidTransform3& CapsuleTransform,
			const TImplicitObjectScaled<FImplicitConvex3>& Convex,
			const FRigidTransform3& ConvexTransform,
			const FReal CullDistance,
			FContactPointManifold& OutContactPoints);

		template
		void ConstructCapsuleConvexOneShotManifold(
			const FImplicitCapsule3& Capsule,
			const FRigidTransform3& CapsuleTransform,
			const FImplicitBox3& Convex,
			const FRigidTransform3& ConvexTransform,
			const FReal CullDistance,
			FContactPointManifold& OutContactPoints);


		template
		void ConstructPlanarConvexTriangleOneShotManifold(
			const FImplicitConvex3& Convex, 
			const FTriangle& Triangle,
			const FReal CullDistance, 
			FContactPointManifold& OutContactPoints);

		template
		void ConstructPlanarConvexTriangleOneShotManifold(
			const TImplicitObjectInstanced<FImplicitConvex3>& Convex,
			const FTriangle& Triangle,
			const FReal CullDistance,
			FContactPointManifold& OutContactPoints);

		template
		void ConstructPlanarConvexTriangleOneShotManifold(
			const TImplicitObjectScaled<FImplicitConvex3>& Convex,
			const FTriangle& Triangle,
			const FReal CullDistance,
			FContactPointManifold& OutContactPoints);

		template
		void ConstructPlanarConvexTriangleOneShotManifold(
			const FImplicitBox3& Convex,
			const FTriangle& Triangle,
			const FReal CullDistance,
			FContactPointManifold& OutContactPoints);

		template
		void ConstructPlanarConvexTriangleOneShotManifold(
			const TImplicitObjectScaled<FImplicitBox3>& Convex,
			const FTriangle& Triangle,
			const FReal CullDistance,
			FContactPointManifold& OutContactPoints);

		template
		void ConstructPlanarConvexTriangleOneShotManifold(
			const TImplicitObjectInstanced<FImplicitBox3>& Convex,
			const FTriangle& Triangle,
			const FReal CullDistance,
			FContactPointManifold& OutContactPoints);


		// @todo(chaos): all ConstructConvexConvexOneShotManifold taking a triangle are still used in FHeightField::ContactManifoldImp and the 
		// TriMesh version, but they should changed to use ConstructPlanarConvexTriangleOneShotManifold
		template
		void ConstructConvexConvexOneShotManifold<FCapsule, FTriangle>(
			const FCapsule& Implicit1,
			const FRigidTransform3& Convex1Transform, //world
			const FTriangle& Implicit2,
			const FRigidTransform3& Convex2Transform, //world
			const FReal Dt,
			FPBDCollisionConstraint& Constraint);

		template
		void ConstructConvexConvexOneShotManifold<TImplicitObjectScaled<FCapsule, 1>, FTriangle>(
			const TImplicitObjectScaled<class FCapsule, 1>& Implicit1,
			const FRigidTransform3& Convex1Transform, //world
			const FTriangle& Implicit2,
			const FRigidTransform3& Convex2Transform, //world
			const FReal Dt,
			FPBDCollisionConstraint& Constraint);

		template
		void ConstructConvexConvexOneShotManifold<FImplicitBox3, FTriangle>(
			const FImplicitBox3& Implicit1,
			const FRigidTransform3& Convex1Transform, //world
			const FTriangle& Implicit2,
			const FRigidTransform3& Convex2Transform, //world
			const FReal Dt,
			FPBDCollisionConstraint& Constraint);

		template
		void ConstructConvexConvexOneShotManifold<FImplicitConvex3, FTriangle>(
			const FImplicitConvex3& Implicit1,
			const FRigidTransform3& Convex1Transform, //world
			const FTriangle& Implicit2,
			const FRigidTransform3& Convex2Transform, //world
			const FReal Dt,
			FPBDCollisionConstraint& Constraint);

		template
		void ConstructConvexConvexOneShotManifold<TImplicitObjectScaled<class TBox<FReal, 3>, 1>, FTriangle>(
			const TImplicitObjectScaled<class Chaos::TBox<FReal, 3>, 1>& Implicit1,
			const FRigidTransform3& Convex1Transform, //world
			const FTriangle& Implicit2,
			const FRigidTransform3& Convex2Transform, //world
			const FReal Dt,
			FPBDCollisionConstraint& Constraint);

		template
		void ConstructConvexConvexOneShotManifold<TImplicitObjectScaled<class FConvex, 1>, FTriangle>(
			const Chaos::TImplicitObjectScaled<class FConvex, 1>& Implicit1,
			const FRigidTransform3& Convex1Transform, //world
			const FTriangle& Implicit2,
			const FRigidTransform3& Convex2Transform, //world
			const FReal Dt,
			FPBDCollisionConstraint& Constraint);

		template
		void ConstructConvexConvexOneShotManifold<class FCapsule, class FConvex>(
			const FCapsule& Implicit1,
			const FRigidTransform3& Convex1Transform, //world
			const FConvex& Implicit2,
			const FRigidTransform3& Convex2Transform, //world
			const FReal Dt,
			FPBDCollisionConstraint& Constraint);

		template
		void ConstructConvexConvexOneShotManifold<class FCapsule, TImplicitObjectScaled<class FConvex, 1>>(
			const FCapsule& Implicit1,
			const FRigidTransform3& Convex1Transform, //world
			const TImplicitObjectScaled<class FConvex, 1>& Implicit2,
			const FRigidTransform3& Convex2Transform, //world
			const FReal Dt,
			FPBDCollisionConstraint& Constraint);

		template
		void ConstructConvexConvexOneShotManifold<class FCapsule, class TImplicitObjectInstanced<class FConvex>>(
			const FCapsule& Implicit1,
			const FRigidTransform3& Convex1Transform, //world
			const TImplicitObjectInstanced<class FConvex>& Implicit2,
			const FRigidTransform3& Convex2Transform, //world
			const FReal Dt,
			FPBDCollisionConstraint& Constraint);

		template
		void ConstructConvexConvexOneShotManifold<class FCapsule, class TBox<FReal, 3>>(
			const FCapsule& Implicit1,
			const FRigidTransform3& Convex1Transform, //world
			const TBox<FReal, 3>& Implicit2,
			const FRigidTransform3& Convex2Transform, //world
			const FReal Dt,
			FPBDCollisionConstraint& Constraint);

		template
		void ConstructConvexConvexOneShotManifold<class FCapsule, TImplicitObjectScaled<class TBox<FReal, 3>, 1>>(
			const FCapsule& Implicit1,
			const FRigidTransform3& Convex1Transform, //world
			const TImplicitObjectScaled<class TBox<FReal, 3>, 1>& Implicit2,
			const FRigidTransform3& Convex2Transform, //world
			const FReal Dt,
			FPBDCollisionConstraint& Constraint);

		template
		void ConstructConvexConvexOneShotManifold<class FCapsule, class TImplicitObjectInstanced<class TBox<FReal, 3>>>(
			const FCapsule& Implicit1,
			const FRigidTransform3& Convex1Transform, //world
			const TImplicitObjectInstanced<class TBox<FReal, 3>>& Implicit2,
			const FRigidTransform3& Convex2Transform, //world
			const FReal Dt,
			FPBDCollisionConstraint& Constraint);
	}
}

