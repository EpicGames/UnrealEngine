// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/Collision/PBDCollisionConstraint.h"
#include "Chaos/Convex.h"
#include "Chaos/ImplicitFwd.h"
#include "Chaos/Defines.h"
#include "Chaos/Transform.h"

namespace Chaos
{
	namespace Collisions
	{
		uint32 BoxBoxClipVerticesAgainstPlane(const FVec3* InputVertexBuffer, FVec3* outputVertexBuffer, uint32 ClipPointCount, int32 ClippingAxis, FReal Distance);
		uint32 ReduceManifoldContactPoints(FVec3* Points, uint32 PointCount);
		uint32 ClipVerticesAgainstPlane(const FVec3* InputVertexBuffer, FVec3* OutputVertexBuffer, const uint32 ClipPointCount, const uint32 MaxNumberOfOutputPoints, const FVec3 ClippingPlaneNormal, const FReal PlaneDistance);

		void ConstructBoxBoxOneShotManifold(
			const FImplicitBox3& Box1,
			const FRigidTransform3& Box1Transform, //world
			const FImplicitBox3& Box2,
			const FRigidTransform3& Box2Transform, //world
			const FReal Dt,
			FPBDCollisionConstraint& Constraint);

		template <typename ConvexImplicitType1, typename ConvexImplicitType2>
		void ConstructConvexConvexOneShotManifold(
			const ConvexImplicitType1& Implicit1,
			const FRigidTransform3& Convex1Transform, //world
			const ConvexImplicitType2& Implicit2,
			const FRigidTransform3& Convex2Transform, //world
			const FReal Dt,
			FPBDCollisionConstraint& Constraint);

		template <typename ConvexType>
		void ConstructCapsuleConvexOneShotManifold(
			const FImplicitCapsule3& Capsule,
			const FRigidTransform3& CapsuleTransform,
			const ConvexType& Convex,
			const FRigidTransform3& ConvexTransform,
			const FReal CullDistance,
			FContactPointManifold& OutContactPoints);

		void ConstructCapsuleTriangleOneShotManifold(
			const FImplicitCapsule3& Capsule,
			const FTriangle& Triangle,
			const FReal CullDistance,
			FContactPointManifold& OutContactPoints);

		template <typename ConvexType>
		void ConstructPlanarConvexTriangleOneShotManifold(
			const ConvexType& Convex, 
			const FTriangle& Triangle, 
			const FReal CullDistance, 
			FContactPointManifold& OutContactPoints);
	}
}


#if 0
DECLARE_STATS_GROUP(TEXT("ChaosManifold"), STATGROUP_ChaosManifold, STATCAT_Advanced);

DECLARE_CYCLE_STAT(TEXT("Manifold::Manifold"), STAT_Collisions_Manifold, STATGROUP_ChaosManifold);
DECLARE_CYCLE_STAT(TEXT("Manifold::ManifoldGJK"), STAT_Collisions_ManifoldGJK, STATGROUP_ChaosManifold);
DECLARE_CYCLE_STAT(TEXT("Manifold::ManifoldEdgeEdge"), STAT_Collisions_ManifoldEdgeEdge, STATGROUP_ChaosManifold);
DECLARE_CYCLE_STAT(TEXT("Manifold::ManifoldClip"), STAT_Collisions_ManifoldClip, STATGROUP_ChaosManifold);
DECLARE_CYCLE_STAT(TEXT("Manifold::ManifoldReduce"), STAT_Collisions_ManifoldReduce, STATGROUP_ChaosManifold);
DECLARE_CYCLE_STAT(TEXT("Manifold::ManifoldFaceVertex"), STAT_Collisions_ManifoldFaceVertex, STATGROUP_ChaosManifold);
#define SCOPE_CYCLE_COUNTER_MANIFOLD() SCOPE_CYCLE_COUNTER(STAT_Collisions_Manifold)\
	PHYSICS_CSV_SCOPED_EXPENSIVE(PhysicsVerbose, NarrowPhase_Manifold)
#define SCOPE_CYCLE_COUNTER_MANIFOLD_GJK() SCOPE_CYCLE_COUNTER(STAT_Collisions_ManifoldGJK)\
	PHYSICS_CSV_SCOPED_EXPENSIVE(PhysicsVerbose, Manifold_GJK)
#define SCOPE_CYCLE_COUNTER_MANIFOLD_ADDEDGEEDGE() SCOPE_CYCLE_COUNTER(STAT_Collisions_ManifoldEdgeEdge)\
	PHYSICS_CSV_SCOPED_EXPENSIVE(PhysicsVerbose, Manifold_EdgeEdge)
#define SCOPE_CYCLE_COUNTER_MANIFOLD_CLIP() SCOPE_CYCLE_COUNTER(STAT_Collisions_ManifoldClip)\
	PHYSICS_CSV_SCOPED_EXPENSIVE(PhysicsVerbose, Manifold_Clip)
#define SCOPE_CYCLE_COUNTER_MANIFOLD_REDUCE() SCOPE_CYCLE_COUNTER(STAT_Collisions_ManifoldReduce)\
	PHYSICS_CSV_SCOPED_EXPENSIVE(PhysicsVerbose, Manifold_Reduce)
#define SCOPE_CYCLE_COUNTER_MANIFOLD_ADDFACEVERTEX() SCOPE_CYCLE_COUNTER(STAT_Collisions_ManifoldFaceVertex)\
	PHYSICS_CSV_SCOPED_EXPENSIVE(PhysicsVerbose, Manifold_FaceVertex)
#elif 1
#define SCOPE_CYCLE_COUNTER_MANIFOLD() PHYSICS_CSV_SCOPED_EXPENSIVE(PhysicsVerbose, NarrowPhase_Manifold)
#define SCOPE_CYCLE_COUNTER_MANIFOLD_GJK() PHYSICS_CSV_SCOPED_EXPENSIVE(PhysicsVerbose, Manifold_GJK)
#define SCOPE_CYCLE_COUNTER_MANIFOLD_ADDEDGEEDGE() PHYSICS_CSV_SCOPED_EXPENSIVE(PhysicsVerbose, Manifold_EdgeEdge)
#define SCOPE_CYCLE_COUNTER_MANIFOLD_CLIP() PHYSICS_CSV_SCOPED_EXPENSIVE(PhysicsVerbose, Manifold_Clip)
#define SCOPE_CYCLE_COUNTER_MANIFOLD_REDUCE() PHYSICS_CSV_SCOPED_EXPENSIVE(PhysicsVerbose, Manifold_Reduce)
#define SCOPE_CYCLE_COUNTER_MANIFOLD_ADDFACEVERTEX() PHYSICS_CSV_SCOPED_EXPENSIVE(PhysicsVerbose, Manifold_FaceVertex)
#else
#define SCOPE_CYCLE_COUNTER_MANIFOLD()
#define SCOPE_CYCLE_COUNTER_MANIFOLD_GJK()
#define SCOPE_CYCLE_COUNTER_MANIFOLD_ADDEDGEEDGE()
#define SCOPE_CYCLE_COUNTER_MANIFOLD_CLIP()
#define SCOPE_CYCLE_COUNTER_MANIFOLD_REDUCE()
#define SCOPE_CYCLE_COUNTER_MANIFOLD_ADDFACEVERTEX()
#endif

