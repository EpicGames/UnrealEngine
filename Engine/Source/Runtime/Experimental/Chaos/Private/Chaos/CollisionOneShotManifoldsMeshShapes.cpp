// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/CollisionOneShotManifoldsMeshShapes.h"

#include "Chaos/ChaosDebugDraw.h"
#include "Chaos/CollisionOneShotManifolds.h"
#include "Chaos/CollisionResolution.h"
#include "Chaos/Collision/CapsuleTriangleContactPoint.h"
#include "Chaos/Collision/ContactPointsMiscShapes.h"
#include "Chaos/Collision/ContactTriangles.h"
#include "Chaos/Collision/ConvexContactPoint.h"
#include "Chaos/Collision/ConvexContactPointUtilities.h"
#include "Chaos/Collision/ConvexTriangleContactPoint.h"
#include "Chaos/Collision/MeshContactGenerator.h"
#include "Chaos/Collision/PBDCollisionConstraint.h"
#include "Chaos/Collision/SphereTriangleContactPoint.h"
#include "Chaos/Convex.h"
#include "Chaos/DebugDrawQueue.h"
#include "Chaos/Defines.h"
#include "Chaos/Framework/UncheckedArray.h"
#include "Chaos/GJK.h"
#include "Chaos/HeightField.h"
#include "Chaos/ImplicitObjectScaled.h"
#include "Chaos/Transform.h"
#include "Chaos/Triangle.h"
#include "Chaos/Utilities.h"
#include "ChaosStats.h"

namespace Chaos
{
	extern bool bChaos_Collision_OneSidedTriangleMesh;
	extern bool bChaos_Collision_OneSidedHeightField;
	extern FRealSingle Chaos_Collision_TriMeshDistanceTolerance;
	extern FRealSingle Chaos_Collision_TriMeshPhiToleranceScale;
	extern int32 Chaos_Collision_MeshManifoldHashSize;
	extern bool bChaos_Collision_EnableMeshManifoldOptimizedLoop;
	extern bool bChaos_Collision_EnableMeshManifoldOptimizedLoop_TriMesh;
	extern bool bChaos_Collision_EnableMACDFallback;
	extern bool bChaos_Collision_EnableMACDPreManifoldFix;

	extern bool bChaos_Collision_UseCapsuleTriMesh2;
	extern int32 Chaos_Collision_ConvexTriMeshMode;
	extern bool bChaos_Collision_ConvexTriMeshInsideCull;
	extern bool bChaos_Collision_ConvexTriMeshBackFaceCull;

	extern bool bChaos_Collision_ConvexTriMeshSortByPhi;
	extern bool bChaos_Collision_ConvexTriMeshSortByDistance;

	namespace CVars
	{
#if CHAOS_DEBUG_DRAW
		extern DebugDraw::FChaosDebugDrawSettings ChaosSolverDebugDebugDrawSettings;
		extern int32 ChaosSolverDebugDrawMeshContacts;
		extern int32 ChaosSolverDebugDrawMeshContactDetails;
#endif
	}

	namespace Collisions
	{

		inline FReal CalculateTriMeshPhiTolerance(const FReal CullDistance)
		{
			return Chaos_Collision_TriMeshPhiToleranceScale * CullDistance;
		}

		template <typename TriMeshType>
		void ConstructSphereTriangleMeshOneShotManifold(const FSphere& Sphere, const FRigidTransform3& SphereWorldTransform, const TriMeshType& TriangleMesh, const FRigidTransform3& TriMeshWorldTransform, const FReal Dt, FPBDCollisionConstraint& Constraint)
		{
			// We only build one shot manifolds once
			// All convexes are pre-scaled, or wrapped in TImplicitObjectScaled
			ensure(SphereWorldTransform.GetScale3D() == FVec3(1.0f, 1.0f, 1.0f));
			ensure(TriMeshWorldTransform.GetScale3D() == FVec3(1.0f, 1.0f, 1.0f));

			// @todo(chaos): support manifold maintenance
			Constraint.ResetActiveManifoldContacts();

			FContactPoint ContactPoint = SphereTriangleMeshContactPoint(Sphere, SphereWorldTransform, TriangleMesh, TriMeshWorldTransform, Constraint.GetCullDistance());
			if (ContactPoint.Phi < Constraint.GetCullDistance())
			{
				Constraint.AddOneshotManifoldContact(ContactPoint);
			}
		}

		void ConstructSphereHeightFieldOneShotManifold(const FSphere& Sphere, const FRigidTransform3& SphereTransform, const FHeightField& Heightfield, const FRigidTransform3& HeightfieldTransform, const FReal Dt, FPBDCollisionConstraint& Constraint)
		{
			// We only build one shot manifolds once
			// All convexes are pre-scaled, or wrapped in TImplicitObjectScaled
			ensure(SphereTransform.GetScale3D() == FVec3(1.0f, 1.0f, 1.0f));
			ensure(HeightfieldTransform.GetScale3D() == FVec3(1.0f, 1.0f, 1.0f));

			FContactPoint ContactPoint = SphereHeightFieldContactPoint(Sphere, SphereTransform, Heightfield, HeightfieldTransform, Constraint.GetCullDistance());
			if (ContactPoint.Phi < Constraint.GetCullDistance())
			{
				Constraint.AddOneshotManifoldContact(ContactPoint);
			}
		}

		template<typename ConvexType>
		void ConstructConvexTriangleOneShotManifold3(
			const ConvexType& Convex,
			const FRigidTransform3& ConvexTransform,
			Private::FMeshContactGenerator& ContactGenerator,
			const int32 TriangleIndex,
			const FReal CullDistance,
			FContactPointManifold& OutContactPoints)
		{
			// Triangle relative to the convex at its predicted position P
			const FTriangle& Triangle = ContactGenerator.GetTriangle(TriangleIndex);
			const FVec3 TriangleNormal = ContactGenerator.GetTriangleNormal(TriangleIndex);

			// If the convex origin is inside the triangle, ignore it
			if (bChaos_Collision_ConvexTriMeshInsideCull)
			{
				const FReal ConvexDistance = FVec3::DotProduct(Triangle.GetCentroid() - Convex.GetCenterOfMass(), TriangleNormal);
				if (ConvexDistance > 0)
				{
					return;
				}
			}

			// Find the closest feature between the Convex at its initial position X and the Triangle
			Private::FConvexContactPoint ClosestContact;
			const bool bFoundClosestContact = Private::FindClosestFeatures(Convex, Triangle, TriangleNormal, FVec3(0), CullDistance, ClosestContact);

			if (bFoundClosestContact)
			{
				ClosestContact.Features[0].ObjectIndex = 0;
				ClosestContact.Features[1].ObjectIndex = TriangleIndex;

				#if CHAOS_DEBUG_DRAW
				if (CVars::ChaosSolverDebugDrawMeshContacts && FDebugDrawQueue::GetInstance().IsDebugDrawingEnabled())
				{
					const FVec3 P = ConvexTransform.TransformPositionNoScale(ClosestContact.ShapeContactPoints[1]);
					const FVec3 N = ConvexTransform.TransformVectorNoScale(ClosestContact.ShapeContactNormal);
					FDebugDrawQueue::GetInstance().DrawDebugLine(P, P + 10.0f * N, FColor::Black, false, CVars::ChaosSolverDebugDebugDrawSettings.DrawDuration, (uint8)CVars::ChaosSolverDebugDebugDrawSettings.DrawPriority, 1.5f * CVars::ChaosSolverDebugDebugDrawSettings.LineThickness);
				}
				#endif

				// Backface cull based on closest contact normal
				const FReal TriangleDotNormal = FVec3::DotProduct(TriangleNormal, ClosestContact.ShapeContactNormal);
				if (bChaos_Collision_ConvexTriMeshBackFaceCull)
				{
					if (TriangleDotNormal < 0)
					{
						return;
					}
				}

				// Cull distance is zero for back faces
				const FReal EffectiveCullDistance = (TriangleDotNormal < 0) ? FReal(0.0) : CullDistance;
				if (ClosestContact.Phi > EffectiveCullDistance)
				{
					return;
				}

				// Use the mesh info to correct the normal - this corrects edge and vertex normals if they are
				// outside the range allowed by the set of triangles sharing the feature
				if (ContactGenerator.FixFeature(TriangleIndex, ClosestContact.Features[1].FeatureType, ClosestContact.Features[1].PlaneFeatureIndex, ClosestContact.ShapeContactNormal))
				{
					// The normal was remapped to the triangle plane
					ClosestContact.Features[0].FeatureType = Private::EConvexFeatureType::Vertex;
					ClosestContact.Features[0].PlaneIndex = Convex.GetMostOpposingPlane(ClosestContact.ShapeContactNormal);
					ClosestContact.Features[0].PlaneFeatureIndex = INDEX_NONE;	// Not needed by ConvexTriangleManifoldFromContact so not worth calculating
				}

				#if CHAOS_DEBUG_DRAW
				if (CVars::ChaosSolverDebugDrawMeshContacts && FDebugDrawQueue::GetInstance().IsDebugDrawingEnabled())
				{
					const FVec3 P = ConvexTransform.TransformPositionNoScale(ClosestContact.ShapeContactPoints[1]);
					const FVec3 N = ConvexTransform.TransformVectorNoScale(ClosestContact.ShapeContactNormal);
					FDebugDrawQueue::GetInstance().DrawDebugLine(P, P + 10.0f * N, FColor::Orange, false, CVars::ChaosSolverDebugDebugDrawSettings.DrawDuration, (uint8)CVars::ChaosSolverDebugDebugDrawSettings.DrawPriority, 1.25f * CVars::ChaosSolverDebugDebugDrawSettings.LineThickness);
				}
				#endif

				// Back face culling based on the corrected feature
				const FReal TriangleDotCorrectedNormal = FVec3::DotProduct(TriangleNormal, ClosestContact.ShapeContactNormal);
				if (TriangleDotCorrectedNormal < 0)
				{
					return;
				}

				// Generate a manifold from on the closest features by projecting the triangle and most opposing convex face onto each other
				Private::ConvexTriangleManifoldFromContact(Convex, Triangle, TriangleNormal, ClosestContact, CullDistance, OutContactPoints);

				#if CHAOS_DEBUG_DRAW
				if (CVars::ChaosSolverDebugDrawMeshContacts && FDebugDrawQueue::GetInstance().IsDebugDrawingEnabled())
				{
					for (int32 ContactIndex = 0; ContactIndex < OutContactPoints.Num(); ++ContactIndex)
					{
						FContactPoint& ContactPoint = OutContactPoints[ContactIndex];
						const FVec3 P = ConvexTransform.TransformPositionNoScale(ContactPoint.ShapeContactPoints[1]);
						const FVec3 N = ConvexTransform.TransformVectorNoScale(ContactPoint.ShapeContactNormal);
						FColor Color = FColor::Black;
						if (ContactPoint.ContactType == EContactPointType::VertexPlane)
						{
							Color = FColor::White;
						}
						else if (ContactPoint.ContactType == EContactPointType::PlaneVertex)
						{
							Color = FColor::Magenta;
						}
						else if (ContactPoint.ContactType == EContactPointType::EdgeEdge)
						{
							Color = FColor::Cyan;
						}
						FDebugDrawQueue::GetInstance().DrawDebugLine(P, P + 10.0f * N, Color, false, CVars::ChaosSolverDebugDebugDrawSettings.DrawDuration, (uint8)CVars::ChaosSolverDebugDebugDrawSettings.DrawPriority, 1.25f * CVars::ChaosSolverDebugDebugDrawSettings.LineThickness);
					}
				}
				#endif
			}
		}

		/**
		 * @brief Generate a manifold between a convex shape and a single triangle
		 * Templated so we can specialize for some shape types
		*/
		template<typename ConvexType>
		void GenerateConvexTriangleOneShotManifold(const ConvexType& Convex, const FTriangle& Triangle, const FReal CullDistance, FContactPointManifold& OutContactPoints)
		{
			if (Chaos_Collision_ConvexTriMeshMode != 0)
			{
				ConstructConvexTriangleOneShotManifold2(Convex, Triangle, CullDistance, OutContactPoints);
			}
			else
			{
				ConstructPlanarConvexTriangleOneShotManifold(Convex, Triangle, CullDistance, OutContactPoints);
			}
		}

		template<>
		void GenerateConvexTriangleOneShotManifold<FImplicitCapsule3>(const FImplicitCapsule3& Capsule, const FTriangle& Triangle, const FReal CullDistance, FContactPointManifold& OutContactPoints)
		{
			if (bChaos_Collision_UseCapsuleTriMesh2)
			{
				ConstructCapsuleTriangleOneShotManifold2(Capsule, Triangle, CullDistance, OutContactPoints);
			}
			else
			{
				ConstructCapsuleTriangleOneShotManifold(Capsule, Triangle, CullDistance, OutContactPoints);
			}
		}

		template<>
		void GenerateConvexTriangleOneShotManifold<FImplicitSphere3>(const FImplicitSphere3& Sphere, const FTriangle& Triangle, const FReal CullDistance, FContactPointManifold& OutContactPoints)
		{
			ConstructSphereTriangleOneShotManifold(Sphere, Triangle, CullDistance, OutContactPoints);
		}

		template<typename ConvexType>
		void GenerateConvexTriangleOneShotManifold(const ConvexType& Convex, const FRigidTransform3& ConvexTransform, Private::FMeshContactGenerator& ContactGenerator, const int32 TriangleIndex, const FReal CullDistance, FContactPointManifold& OutContactPoints)
		{
			if (Chaos_Collision_ConvexTriMeshMode == 2)
			{
				ConstructConvexTriangleOneShotManifold3(Convex, ConvexTransform, ContactGenerator, TriangleIndex, CullDistance, OutContactPoints);
				ContactGenerator.SetFixNormalsEnabled(false);
			}
			else if (Chaos_Collision_ConvexTriMeshMode == 1)
			{
				const FTriangle& Triangle = ContactGenerator.GetTriangle(TriangleIndex);
				ConstructConvexTriangleOneShotManifold2(Convex, Triangle, CullDistance, OutContactPoints);
			}
			else
			{
				const FTriangle& Triangle = ContactGenerator.GetTriangle(TriangleIndex);
				ConstructPlanarConvexTriangleOneShotManifold(Convex, Triangle, CullDistance, OutContactPoints);
			}
		}

		template<>
		void GenerateConvexTriangleOneShotManifold<FImplicitCapsule3>(const FImplicitCapsule3& Capsule, const FRigidTransform3& ConvexTransform, Private::FMeshContactGenerator& ContactGenerator, const int32 TriangleIndex, const FReal CullDistance, FContactPointManifold& OutContactPoints)
		{
			const FTriangle& Triangle = ContactGenerator.GetTriangle(TriangleIndex);
			if (bChaos_Collision_UseCapsuleTriMesh2)
			{
				ConstructCapsuleTriangleOneShotManifold2(Capsule, Triangle, CullDistance, OutContactPoints);
			}
			else
			{
				ConstructCapsuleTriangleOneShotManifold(Capsule, Triangle, CullDistance, OutContactPoints);
			}
		}

		template<>
		void GenerateConvexTriangleOneShotManifold<FImplicitSphere3>(const FImplicitSphere3& Sphere, const FRigidTransform3& ConvexTransform, Private::FMeshContactGenerator& ContactGenerator, const int32 TriangleIndex, const FReal CullDistance, FContactPointManifold& OutContactPoints)
		{
			const FTriangle& Triangle = ContactGenerator.GetTriangle(TriangleIndex);
			ConstructSphereTriangleOneShotManifold(Sphere, Triangle, CullDistance, OutContactPoints);
		}

		template<typename ConvexType, typename MeshType>
		void ConstructConvexMeshOneShotManifold2(const ConvexType& Convex, const FRigidTransform3& ConvexTransform, const MeshType& Mesh, const FRigidTransform3& MeshTransform, const FVec3& MeshScale, const FReal CullDistance, Private::FMeshContactGenerator& ContactGenerator)
		{
			FRigidTransform3 MeshToConvexTransform = MeshTransform.GetRelativeTransformNoScale(ConvexTransform);
			MeshToConvexTransform.SetScale3D(MeshScale);

			// @todo(chaos): add Convex.CalculateInverseTransformed bounds with scale support (to optimize sphere and capsule)
			const FAABB3 ConvexBounds = FAABB3(Convex.BoundingBox()).Thicken(CullDistance);
			const FAABB3 MeshQueryBounds = ConvexBounds.InverseTransformedAABB(MeshToConvexTransform);

			// Generate the contact manifold between Convex and a Triangle
			const auto& GenerateConvexTriangleContacts =
				[&Convex, &ConvexTransform, CullDistance](Private::FMeshContactGenerator& ContactGenerator, const int32 TriangleIndex)
			{
				FContactPointManifold Contacts;
				GenerateConvexTriangleOneShotManifold(Convex, ConvexTransform, ContactGenerator, TriangleIndex, CullDistance, Contacts);

				ContactGenerator.AddTriangleContacts(TriangleIndex, MakeArrayView(Contacts));
			};

			// Collect all the triangles that overlap our convex. Triangles will be in Convex space
			Mesh.CollectTriangles(MeshQueryBounds, MeshToConvexTransform, ConvexBounds, ContactGenerator);

			// Generate a set of contact points for all triangles
			ContactGenerator.GenerateMeshContacts(GenerateConvexTriangleContacts);

			// Process the contacts to minimize manifold etc
			ContactGenerator.ProcessGeneratedContacts(ConvexTransform, MeshToConvexTransform);
		}

		// Original MACD algorithm that uses mesh information to fix manifold point normals after the manifold is built.
		// @todo(chaos): remove this when the new version is well tested.
		template<typename ConvexType>
		void GenerateConvexTriangleOneShotManifoldMACD_PostManifoldFix(const ConvexType& Convex, const FRigidTransform3& ConvexTransform, const FVec3& InConvexRelativeMovement, Private::FMeshContactGenerator& ContactGenerator, const int32 TriangleIndex, const FReal CullDistance, FContactPointManifold& OutContactPoints)
		{
			FTriangle Triangle = ContactGenerator.GetTriangle(TriangleIndex);
			const FVec3 TriangleNormal = ContactGenerator.GetTriangleNormal(TriangleIndex);
			const FReal ConvexTriangleDistanceAtP = FVec3::DotProduct(Convex.GetCenterOfMass() - Triangle.GetVertex(0), TriangleNormal);

			// If we are outside the plane of the triangle at P, collide at P
			bool bUseMACD = !InConvexRelativeMovement.IsZero() && (ConvexTriangleDistanceAtP < 0);
			
			// If desired, shift the triangle so it is relative to the convex when it was at X
			FVec3 ConvexRelativeMovement = FVec3(0);
			FReal ConvexTriangleDistance = ConvexTriangleDistanceAtP;
			if (bUseMACD)
			{
				ConvexRelativeMovement = InConvexRelativeMovement;
				Triangle[0] += ConvexRelativeMovement;
				Triangle[1] += ConvexRelativeMovement;
				Triangle[2] += ConvexRelativeMovement;
				ConvexTriangleDistance -= FVec3::DotProduct(ConvexRelativeMovement, TriangleNormal);
			}

			// If we were inside the triangle at X and P we ignore this triangle
			if ((ConvexTriangleDistance < 0) && (ConvexTriangleDistanceAtP < 0))
			{
				return;
			}

			// Find the closest feature pair on the triangle and convex
			Private::FConvexContactPoint ClosestContact;
			if (Private::FindClosestFeatures(Convex, Triangle, TriangleNormal, ConvexRelativeMovement, CullDistance, ClosestContact))
			{
				ClosestContact.Features[0].ObjectIndex = 0;
				ClosestContact.Features[1].ObjectIndex = TriangleIndex;

#if CHAOS_DEBUG_DRAW
				if (CVars::ChaosSolverDebugDrawMeshContacts && FDebugDrawQueue::GetInstance().IsDebugDrawingEnabled())
				{
					const FVec3 P = ConvexTransform.TransformPositionNoScale(ClosestContact.ShapeContactPoints[1] - ConvexRelativeMovement);
					const FVec3 N = ConvexTransform.TransformVectorNoScale(ClosestContact.ShapeContactNormal);
					FDebugDrawQueue::GetInstance().DrawDebugLine(P, P + 10.0f * N, FColor::Black, false, CVars::ChaosSolverDebugDebugDrawSettings.DrawDuration, (uint8)CVars::ChaosSolverDebugDebugDrawSettings.DrawPriority, 1.25f * CVars::ChaosSolverDebugDebugDrawSettings.LineThickness);
				}
				if (CVars::ChaosSolverDebugDrawMeshContactDetails && FDebugDrawQueue::GetInstance().IsDebugDrawingEnabled())
				{
					const FVec3 XConvex = ConvexTransform.TransformPositionNoScale(-ConvexRelativeMovement);
					const FColor Color = (bUseMACD) ? FColor::Red : FColor::Green;
					FDebugDrawQueue::GetInstance().DrawDebugPoint(XConvex, Color, false, CVars::ChaosSolverDebugDebugDrawSettings.DrawDuration, (uint8)CVars::ChaosSolverDebugDebugDrawSettings.DrawPriority, 20.0f * CVars::ChaosSolverDebugDebugDrawSettings.LineThickness);
				}
#endif

				// Back face culling
				const FReal TriangleDotNormal = FVec3::DotProduct(TriangleNormal, ClosestContact.ShapeContactNormal);
				if (TriangleDotNormal < 0)
				{
					return;
				}

				// Calculate cull distance that takes movement direction and distance into account
				FReal NetCullDistance = CullDistance;
				if (bUseMACD)
				{
					const FReal ConvexMotionAlongNormal = FVec3::DotProduct(ConvexRelativeMovement, ClosestContact.ShapeContactNormal);
					const FReal CullDistancePadding = FMath::Max(0, -ConvexMotionAlongNormal);
					NetCullDistance += CullDistancePadding;
				}

				// Generate a manifold based on the closest features
				// NOTE: normal points from triangle to convex
				Private::ConvexTriangleManifoldFromContact(Convex, Triangle, TriangleNormal, ClosestContact, NetCullDistance, OutContactPoints);

				if (OutContactPoints.Num() > 0)
				{
#if CHAOS_DEBUG_DRAW
					if (CVars::ChaosSolverDebugDrawMeshContacts && FDebugDrawQueue::GetInstance().IsDebugDrawingEnabled())
					{
						for (int32 ContactIndex = 0; ContactIndex < OutContactPoints.Num(); ++ContactIndex)
						{
							FContactPoint& ContactPoint = OutContactPoints[ContactIndex];
							const FVec3 P = ConvexTransform.TransformPositionNoScale(ContactPoint.ShapeContactPoints[1] - ConvexRelativeMovement);
							const FVec3 N = ConvexTransform.TransformVectorNoScale(ContactPoint.ShapeContactNormal);
							FColor Color = FColor::Black;
							switch (ClosestContact.Features[1].FeatureType)
							{
							case Private::EConvexFeatureType::Plane:
								Color = FColor::White;
								break;
							case Private::EConvexFeatureType::Edge:
								Color = FColor::Cyan;
								break;
							case Private::EConvexFeatureType::Vertex:
								Color = FColor::Magenta;
								break;
							}
							FDebugDrawQueue::GetInstance().DrawDebugLine(P, P + 10.0f * N, Color, false, CVars::ChaosSolverDebugDebugDrawSettings.DrawDuration, (uint8)CVars::ChaosSolverDebugDebugDrawSettings.DrawPriority, 1.25f * CVars::ChaosSolverDebugDebugDrawSettings.LineThickness);
						}
					}
#endif

					// Adjust the closest feature if it is invalid. E.g., if we collide with a triangle edge and the normal is outside the range allowed by
					// the triangles sharing the edge we will project the normal into the valid range. If the feature gets chnaged, we will alter all
					// of the manifold points to use the new normal.
					const bool bFeatureChanged = ContactGenerator.FixFeature(TriangleIndex, ClosestContact.Features[1].FeatureType, ClosestContact.Features[1].PlaneFeatureIndex, ClosestContact.ShapeContactNormal);
					if (bFeatureChanged)
					{
						for (int32 ContactIndex = 0; ContactIndex < OutContactPoints.Num(); ++ContactIndex)
						{
							FContactPoint& ContactPoint = OutContactPoints[ContactIndex];

							// Update the normal and recalculate the separation
							ContactPoint.ShapeContactNormal = ClosestContact.ShapeContactNormal;
							ContactPoint.Phi = FVec3::DotProduct(ContactPoint.ShapeContactPoints[0] - ContactPoint.ShapeContactPoints[1], ContactPoint.ShapeContactNormal);

							// Remap the triangle contact onto the new plane, keeping the contact point on the convex shape where it is.
							// NOTE: This means that the triangle contact point may be outside the triangle, but for contact separation
							// we really only care about the contact plane. This is required for static friction, which assumes the contacts 
							// have zero tangential separation on the frame they are generated.
							ContactPoint.ShapeContactPoints[1] = ContactPoint.ShapeContactPoints[0] - (ContactPoint.Phi * ContactPoint.ShapeContactNormal);
						}
					}

					// Correct the contact points if we ran collision detection at X rather than P
					if (bUseMACD)
					{
						for (int32 ContactIndex = 0; ContactIndex < OutContactPoints.Num(); ++ContactIndex)
						{
							FContactPoint& ContactPoint = OutContactPoints[ContactIndex];

							const FReal ShiftDotNormal = FVec3::DotProduct(ConvexRelativeMovement, ContactPoint.ShapeContactNormal);
							ContactPoint.ShapeContactPoints[1] += -ShiftDotNormal * ContactPoint.ShapeContactNormal;
							ContactPoint.Phi += ShiftDotNormal;
						}
					}
				}
			}
		}

		// MACD: Motion-Aware Collision Detection
		//
		// We have a convex moving from position X to P in this tick.
		// 
		// Detect collisions between the convex and a triangle taking that motion into account.
		// 
		// Find the closest features between the Convex at X and the Triangle. Use those features
		// to select the Convex and Triangle Faces that will be projected onto each other to form
		// the manifold. 
		// 
		// Use the Mesh information to correct the Triangle feature so that the normal is within the
		// valid range. Edge and Vertex collisions with normals outside their valid ranges (determined
		// by the other triangles that share the edge/vertex) are converted to face collisions.
		// 
		// As long as the Convex starts off outside the Triangle, we will generate useful contacts, 
		// even if the Convex is fully inside the triangle at P (this is where the non-MACD path 
		// would fail since it detects collisions only at P).
		// 
		// To generate the best manifold we select a point along the X-P trajectory that is
		// closest to the Triangle's Axis (line though its centroid along its normal). See 
		// amazing ascii art below. The box should collide with the triangle but clipping
		// the triangle to the bottom box face would lead to no contacts at both X and P.
		// 
		//        +--------+
		//  X:    |        |
		//        +--------+
		//
		//  Tri:             ---------
		//
		//                                +--------+
		//  P:                            |        |
		//                                +--------+
		template<typename ConvexType>
		void GenerateConvexTriangleOneShotManifoldMACD_PreManifoldFix(
			const ConvexType& Convex, 
			const FRigidTransform3& ConvexTransform, 
			const FVec3& ConvexRelativeMovement, 
			Private::FMeshContactGenerator& ContactGenerator, 
			const int32 TriangleIndex, 
			const FReal CullDistance, 
			FContactPointManifold& OutContactPoints)
		{
			// NOTE: The triangles were generated in Convex space with the convex at its predicted position P. I.e., we are in the space where P = 0.
			// The convex moved from its initial position X to its predicted position P, and P = X + ConvexRelativeMovement.

			// Triangle relative to the convex at its predicted position P
			const FTriangle TriangleP = ContactGenerator.GetTriangle(TriangleIndex);
			const FVec3 TriangleNormal = ContactGenerator.GetTriangleNormal(TriangleIndex);
			const FVec3 TriangleCentroidP = TriangleP.GetCentroid();

			// If we started inside the triangle we ignore this triangle
			const FReal ConvexRelativeMovementTriNormal = FVec3::DotProduct(ConvexRelativeMovement, TriangleNormal);
			const FReal ConvexTriangleDistanceX = FVec3::DotProduct(Convex.GetCenterOfMass() - TriangleP.GetVertex(0), TriangleNormal) - ConvexRelativeMovementTriNormal;
			if (ConvexTriangleDistanceX < 0)
			{
				return;
			}

			// Triangle relative to the convex at its initial position X
			// NOTE: we do not move the convex, we move the triangle to the relative position as if the convex were moved by -ConvexRelativeMovement.
			const FVec3 TriangleXShift = ConvexRelativeMovement;
			FTriangle TriangleX = FTriangle(TriangleP.GetVertex(0) + TriangleXShift, TriangleP.GetVertex(1) + TriangleXShift, TriangleP.GetVertex(2) + TriangleXShift);

			// Find the closest feature between the Convex at its initial position X and the Triangle
			Private::FConvexContactPoint ClosestContact;
			const bool bFoundClosestContact = Private::FindClosestFeatures(Convex, TriangleX, TriangleNormal, ConvexRelativeMovement, CullDistance, ClosestContact);

			if (bFoundClosestContact)
			{
				ClosestContact.Features[0].ObjectIndex = 0;
				ClosestContact.Features[1].ObjectIndex = TriangleIndex;

#if CHAOS_DEBUG_DRAW
				if (CVars::ChaosSolverDebugDrawMeshContacts && FDebugDrawQueue::GetInstance().IsDebugDrawingEnabled())
				{
					const FVec3 P = ConvexTransform.TransformPositionNoScale(ClosestContact.ShapeContactPoints[1] - ConvexRelativeMovement);
					const FVec3 N = ConvexTransform.TransformVectorNoScale(ClosestContact.ShapeContactNormal);
					FDebugDrawQueue::GetInstance().DrawDebugLine(P, P + 10.0f * N, FColor::Black, false, CVars::ChaosSolverDebugDebugDrawSettings.DrawDuration, (uint8)CVars::ChaosSolverDebugDebugDrawSettings.DrawPriority, 1.5f * CVars::ChaosSolverDebugDebugDrawSettings.LineThickness);
				}
#endif

				// Use the mesh info to correct the normal - this corrects edge and vertex normals if they are
				// outside the range allowed by the set of triangles sharing the feature
				if (ContactGenerator.FixFeature(TriangleIndex, ClosestContact.Features[1].FeatureType, ClosestContact.Features[1].PlaneFeatureIndex, ClosestContact.ShapeContactNormal))
				{
					// The normal was remapped to the triangle plane
					ClosestContact.Features[0].FeatureType = Private::EConvexFeatureType::Vertex;
					ClosestContact.Features[0].PlaneIndex = Convex.GetMostOpposingPlane(ClosestContact.ShapeContactNormal);
					ClosestContact.Features[0].PlaneFeatureIndex = INDEX_NONE;	// Not needed by ConvexTriangleManifoldFromContact so not worth calculating
				}

				// Back face culling based on the corrected feature
				const FReal TriangleDotNormal = FVec3::DotProduct(TriangleNormal, ClosestContact.ShapeContactNormal);
				if (TriangleDotNormal < 0)
				{
					return;
				}

				// We will detect collisions at some point as the convex moves from X to P. 
				// The point we choose is the closest approach to the axis along the triangle normal through the triangle centroid.
				FReal ConvexT, TriangleT;
				FVec3 ConvexNearPos, TriangleNearPos;
				const FVec3 TriangleCentroidX = TriangleCentroidP + ConvexRelativeMovement;
				Utilities::NearestPointsOnLineSegmentToLine(
					FVec3(0), ConvexRelativeMovement,
					TriangleCentroidX, TriangleNormal,
					ConvexT, TriangleT, ConvexNearPos, TriangleNearPos);

				// Triangle relative to the convex at ConvexNearPos
				const FVec3 TriangleNearestShift = ConvexRelativeMovement - ConvexNearPos;
				FTriangle TriangleNearest = FTriangle(TriangleP.GetVertex(0) + TriangleNearestShift, TriangleP.GetVertex(1) + TriangleNearestShift, TriangleP.GetVertex(2) + TriangleNearestShift);

#if CHAOS_DEBUG_DRAW
				if (CVars::ChaosSolverDebugDrawMeshContacts && FDebugDrawQueue::GetInstance().IsDebugDrawingEnabled())
				{
					const FVec3 P = ConvexTransform.TransformPositionNoScale(ClosestContact.ShapeContactPoints[1] - ConvexRelativeMovement);
					const FVec3 N = ConvexTransform.TransformVectorNoScale(ClosestContact.ShapeContactNormal);
					FDebugDrawQueue::GetInstance().DrawDebugLine(P, P + 10.0f * N, FColor::Orange, false, CVars::ChaosSolverDebugDebugDrawSettings.DrawDuration, (uint8)CVars::ChaosSolverDebugDebugDrawSettings.DrawPriority, 1.25f * CVars::ChaosSolverDebugDebugDrawSettings.LineThickness);

					const FVec3 X0 = ConvexTransform.TransformPositionNoScale(Convex.GetCenterOfMass() - ConvexRelativeMovement);
					const FVec3 X1 = ConvexTransform.TransformPositionNoScale(Convex.GetCenterOfMass());
					FDebugDrawQueue::GetInstance().DrawDebugLine(X0, X1, FColor::White, false, CVars::ChaosSolverDebugDebugDrawSettings.DrawDuration, (uint8)CVars::ChaosSolverDebugDebugDrawSettings.DrawPriority, 0.5f * CVars::ChaosSolverDebugDebugDrawSettings.LineThickness);
				}
				if (CVars::ChaosSolverDebugDrawMeshContactDetails && FDebugDrawQueue::GetInstance().IsDebugDrawingEnabled())
				{
					const FVec3 TriangleAxis = FMath::Abs(ConvexRelativeMovementTriNormal) * TriangleNormal;
					const FVec3 TA0 = ConvexTransform.TransformPositionNoScale(TriangleCentroidX + TriangleAxis - ConvexRelativeMovement);
					const FVec3 TA1 = ConvexTransform.TransformPositionNoScale(TriangleCentroidX - TriangleAxis - ConvexRelativeMovement);
					const FVec3 XConvex = ConvexTransform.TransformPositionNoScale(ConvexNearPos - ConvexRelativeMovement);
					const FVec3 XTriangle = ConvexTransform.TransformPositionNoScale(TriangleNearPos - ConvexRelativeMovement);
					FDebugDrawQueue::GetInstance().DrawDebugLine(TA0, TA1, FColor::Black, false, CVars::ChaosSolverDebugDebugDrawSettings.DrawDuration, (uint8)CVars::ChaosSolverDebugDebugDrawSettings.DrawPriority, 0.5f * CVars::ChaosSolverDebugDebugDrawSettings.LineThickness);
					FDebugDrawQueue::GetInstance().DrawDebugLine(XConvex, XTriangle, FColor::Black, false, CVars::ChaosSolverDebugDebugDrawSettings.DrawDuration, (uint8)CVars::ChaosSolverDebugDebugDrawSettings.DrawPriority, 0.5f * CVars::ChaosSolverDebugDebugDrawSettings.LineThickness);
					FDebugDrawQueue::GetInstance().DrawDebugPoint(XConvex, FColor::White, false, CVars::ChaosSolverDebugDebugDrawSettings.DrawDuration, (uint8)CVars::ChaosSolverDebugDebugDrawSettings.DrawPriority, 20.0f * CVars::ChaosSolverDebugDebugDrawSettings.LineThickness);

					const FVec3 XCentroid = ConvexTransform.TransformPositionNoScale(TriangleCentroidX - ConvexRelativeMovement);
					const FMatrix TriangleMat = FRotationMatrix::MakeFromZ(ConvexTransform.TransformVectorNoScale(TriangleNormal));
					FDebugDrawQueue::GetInstance().DrawDebugCircle(XCentroid, 2.0f, 8, FColor::Black, false, CVars::ChaosSolverDebugDebugDrawSettings.DrawDuration, (uint8)CVars::ChaosSolverDebugDebugDrawSettings.DrawPriority, 0.5f * CVars::ChaosSolverDebugDebugDrawSettings.LineThickness, TriangleMat.GetUnitAxis(EAxis::X), TriangleMat.GetUnitAxis(EAxis::Y), false);
				}
#endif

				// Generate a manifold based on the closest features
				// NOTE: normal points from triangle to convex
				const FReal ConvexMotionAlongNormal = FVec3::DotProduct(ConvexRelativeMovement, ClosestContact.ShapeContactNormal);
				const FReal CullDistancePadding = FMath::Max(0, -ConvexMotionAlongNormal);
				const FReal NetCullDistance = CullDistance + CullDistancePadding;
				Private::ConvexTriangleManifoldFromContact(Convex, TriangleNearest, TriangleNormal, ClosestContact, NetCullDistance, OutContactPoints);

				if (OutContactPoints.Num() > 0)
				{
#if CHAOS_DEBUG_DRAW
					if (CVars::ChaosSolverDebugDrawMeshContacts && FDebugDrawQueue::GetInstance().IsDebugDrawingEnabled())
					{
						for (int32 ContactIndex = 0; ContactIndex < OutContactPoints.Num(); ++ContactIndex)
						{
							FContactPoint& ContactPoint = OutContactPoints[ContactIndex];
							const FVec3 P = ConvexTransform.TransformPositionNoScale(ContactPoint.ShapeContactPoints[1] - TriangleNearestShift);
							const FVec3 N = ConvexTransform.TransformVectorNoScale(ContactPoint.ShapeContactNormal);
							FColor Color = FColor::Black;
							switch (ClosestContact.Features[1].FeatureType)
							{
							case Private::EConvexFeatureType::Plane:
								Color = FColor::White;
								break;
							case Private::EConvexFeatureType::Edge:
								Color = FColor::Cyan;
								break;
							case Private::EConvexFeatureType::Vertex:
								Color = FColor::Magenta;
								break;
							}
							FDebugDrawQueue::GetInstance().DrawDebugLine(P, P + 10.0f * N, Color, false, CVars::ChaosSolverDebugDebugDrawSettings.DrawDuration, (uint8)CVars::ChaosSolverDebugDebugDrawSettings.DrawPriority, 1.25f * CVars::ChaosSolverDebugDebugDrawSettings.LineThickness);
						}
					}
#endif

					// Correct the contact points based on convex movement
					for (int32 ContactIndex = 0; ContactIndex < OutContactPoints.Num(); ++ContactIndex)
					{
						FContactPoint& ContactPoint = OutContactPoints[ContactIndex];
						
						const FReal ShiftDotNormal = FVec3::DotProduct(TriangleNearestShift, ContactPoint.ShapeContactNormal);
						ContactPoint.ShapeContactPoints[1] += -ShiftDotNormal * ContactPoint.ShapeContactNormal;
						ContactPoint.Phi += ShiftDotNormal;
					}
				}
			}
		}

		template<typename ConvexType>
		void GenerateConvexTriangleOneShotManifoldMACD(
			const ConvexType& Convex,
			const FRigidTransform3& ConvexTransform,
			const FVec3& ConvexRelativeMovement,
			Private::FMeshContactGenerator& ContactGenerator,
			const int32 TriangleIndex,
			const FReal CullDistance,
			FContactPointManifold& OutContactPoints)
		{
			if (bChaos_Collision_EnableMACDPreManifoldFix)
			{
				GenerateConvexTriangleOneShotManifoldMACD_PreManifoldFix(Convex, ConvexTransform, ConvexRelativeMovement, ContactGenerator, TriangleIndex, CullDistance, OutContactPoints);
			}
			else
			{
				GenerateConvexTriangleOneShotManifoldMACD_PostManifoldFix(Convex, ConvexTransform, ConvexRelativeMovement, ContactGenerator, TriangleIndex, CullDistance, OutContactPoints);
			}
		}

		template<>
		void GenerateConvexTriangleOneShotManifoldMACD(
			const FImplicitSphere3& Convex,
			const FRigidTransform3& ConvexTransform,
			const FVec3& InConvexRelativeMovement,
			Private::FMeshContactGenerator& ContactGenerator,
			const int32 TriangleIndex,
			const FReal CullDistance,
			FContactPointManifold& OutContactPoints)
		{
			FTriangle Triangle = ContactGenerator.GetTriangle(TriangleIndex);
			const FVec3 TriangleNormal = ContactGenerator.GetTriangleNormal(TriangleIndex);
			const FReal ConvexTriangleDistanceAtP = FVec3::DotProduct(Convex.GetCenterOfMass() - Triangle.GetVertex(0), TriangleNormal);

			// If we are outside the plane of the triangle at P, collide at P
			bool bUseMACD = !InConvexRelativeMovement.IsZero() && (ConvexTriangleDistanceAtP < 0);

			// If desired, shift the triangle so it is relative to the convex when it was at X
			FVec3 ConvexRelativeMovement = FVec3(0);
			FReal ConvexTriangleDistance = ConvexTriangleDistanceAtP;
			if (bUseMACD)
			{
				ConvexRelativeMovement = InConvexRelativeMovement;
				Triangle[0] += ConvexRelativeMovement;
				Triangle[1] += ConvexRelativeMovement;
				Triangle[2] += ConvexRelativeMovement;
				ConvexTriangleDistance -= FVec3::DotProduct(ConvexRelativeMovement, TriangleNormal);
			}

			// If we were inside the triangle at X and P we ignore this triangle
			if ((ConvexTriangleDistance < 0) && (ConvexTriangleDistanceAtP < 0))
			{
				return;
			}

			// Find the closest feature pair on the triangle and convex
			Private::FConvexContactPoint ClosestContact;
			if (Private::FindClosestFeatures(Convex, Triangle, TriangleNormal, ConvexRelativeMovement, CullDistance, ClosestContact))
			{
				ClosestContact.Features[0].ObjectIndex = 0;
				ClosestContact.Features[1].ObjectIndex = TriangleIndex;

#if CHAOS_DEBUG_DRAW
				if (CVars::ChaosSolverDebugDrawMeshContacts && FDebugDrawQueue::GetInstance().IsDebugDrawingEnabled())
				{
					const FVec3 P = ConvexTransform.TransformPositionNoScale(ClosestContact.ShapeContactPoints[1] - ConvexRelativeMovement);
					const FVec3 N = ConvexTransform.TransformVectorNoScale(ClosestContact.ShapeContactNormal);
					FDebugDrawQueue::GetInstance().DrawDebugLine(P, P + 10.0f * N, FColor::Black, false, CVars::ChaosSolverDebugDebugDrawSettings.DrawDuration, (uint8)CVars::ChaosSolverDebugDebugDrawSettings.DrawPriority, 1.25f * CVars::ChaosSolverDebugDebugDrawSettings.LineThickness);
				}
				if (CVars::ChaosSolverDebugDrawMeshContactDetails && FDebugDrawQueue::GetInstance().IsDebugDrawingEnabled())
				{
					const FVec3 XConvex = ConvexTransform.TransformPositionNoScale(-ConvexRelativeMovement);
					const FColor Color = (bUseMACD) ? FColor::Red : FColor::Green;
					FDebugDrawQueue::GetInstance().DrawDebugPoint(XConvex, Color, false, CVars::ChaosSolverDebugDebugDrawSettings.DrawDuration, (uint8)CVars::ChaosSolverDebugDebugDrawSettings.DrawPriority, 20.0f * CVars::ChaosSolverDebugDebugDrawSettings.LineThickness);
				}
#endif

				// Back face culling
				const FReal TriangleDotNormal = FVec3::DotProduct(TriangleNormal, ClosestContact.ShapeContactNormal);
				if (TriangleDotNormal < 0)
				{
					return;
				}

				FContactPoint& Contact = OutContactPoints[OutContactPoints.AddUninitialized()];
				Contact.ShapeContactPoints[0] = ClosestContact.ShapeContactPoints[0];
				Contact.ShapeContactPoints[1] = ClosestContact.ShapeContactPoints[1];
				Contact.ShapeContactNormal = ClosestContact.ShapeContactNormal;
				Contact.Phi = ClosestContact.Phi;
				Contact.ContactType = ClosestContact.GetContactPointType();
				Contact.FaceIndex = INDEX_NONE;

#if CHAOS_DEBUG_DRAW
				if (CVars::ChaosSolverDebugDrawMeshContacts && FDebugDrawQueue::GetInstance().IsDebugDrawingEnabled())
				{
					for (int32 ContactIndex = 0; ContactIndex < OutContactPoints.Num(); ++ContactIndex)
					{
						FContactPoint& ContactPoint = OutContactPoints[ContactIndex];
						const FVec3 P = ConvexTransform.TransformPositionNoScale(ContactPoint.ShapeContactPoints[1] - ConvexRelativeMovement);
						const FVec3 N = ConvexTransform.TransformVectorNoScale(ContactPoint.ShapeContactNormal);
						FColor Color = FColor::Black;
						switch (ClosestContact.Features[1].FeatureType)
						{
						case Private::EConvexFeatureType::Plane:
							Color = FColor::White;
							break;
						case Private::EConvexFeatureType::Edge:
							Color = FColor::Cyan;
							break;
						case Private::EConvexFeatureType::Vertex:
							Color = FColor::Magenta;
							break;
						}
						FDebugDrawQueue::GetInstance().DrawDebugLine(P, P + 10.0f * N, Color, false, CVars::ChaosSolverDebugDebugDrawSettings.DrawDuration, (uint8)CVars::ChaosSolverDebugDebugDrawSettings.DrawPriority, 1.25f * CVars::ChaosSolverDebugDebugDrawSettings.LineThickness);
					}
				}
#endif

				// Adjust the closest feature if it is invalid. E.g., if we collide with a triangle edge and the normal is outside the range allowed by
				// the triangles sharing the edge we will project the normal into the valid range. If the feature gets chnaged, we will alter all
				// of the manifold points to use the new normal.
				const bool bFeatureChanged = ContactGenerator.FixFeature(TriangleIndex, ClosestContact.Features[1].FeatureType, ClosestContact.Features[1].PlaneFeatureIndex, ClosestContact.ShapeContactNormal);
				if (bFeatureChanged)
				{
					FContactPoint& ContactPoint = OutContactPoints[0];

					// Update the normal and recalculate the separation
					ContactPoint.ShapeContactNormal = ClosestContact.ShapeContactNormal;
					ContactPoint.Phi = FVec3::DotProduct(ContactPoint.ShapeContactPoints[0] - ContactPoint.ShapeContactPoints[1], ContactPoint.ShapeContactNormal);

					// Remap the triangle contact onto the new plane, keeping the contact point on the convex shape where it is.
					// NOTE: This means that the triangle contact point may be outside the triangle, but for contact separation
					// we really only care about the contact plane. This is required for static friction, which assumes the contacts 
					// have zero tangential separation on the frame they are generated.
					ContactPoint.ShapeContactPoints[1] = ContactPoint.ShapeContactPoints[0] - (ContactPoint.Phi * ContactPoint.ShapeContactNormal);
				}

				// Correct the contact points if we ran collision detection at X rather than P
				if (bUseMACD)
				{
					FContactPoint& ContactPoint = OutContactPoints[0];

					const FReal ShiftDotNormal = FVec3::DotProduct(ConvexRelativeMovement, ContactPoint.ShapeContactNormal);
					ContactPoint.ShapeContactPoints[1] += -ShiftDotNormal * ContactPoint.ShapeContactNormal;
					ContactPoint.Phi += ShiftDotNormal;
				}
			}
		}
		// MACD: Motion-Aware Collision Detection
		template<typename ConvexType, typename MeshType>
		void ConstructConvexMeshOneShotManifoldMACD(
			const ConvexType& Convex, 
			const FRigidTransform3& ConvexTransform, 
			const MeshType& Mesh, 
			const FRigidTransform3& MeshTransform, 
			const FVec3& MeshScale, 
			const FVec3& RelativeMovement, 
			const FReal InCullDistance, 
			Private::FMeshContactGenerator& ContactGenerator)
		{
			if (RelativeMovement.IsZero())
			{
				ConstructConvexMeshOneShotManifold2(Convex, ConvexTransform, Mesh, MeshTransform, MeshScale, InCullDistance, ContactGenerator);
			}
			else
			{
				FRigidTransform3 MeshToConvexTransform = MeshTransform.GetRelativeTransformNoScale(ConvexTransform);
				MeshToConvexTransform.SetScale3D(MeshScale);

				// NOTE: Convex bounds is extended backwards to encompass the pre-movement position
				const FVec3 ConvexRelativeMovement = ConvexTransform.InverseTransformVectorNoScale(RelativeMovement);
				const FAABB3 ConvexBounds = FAABB3(Convex.BoundingBox()).GrowByVector(-ConvexRelativeMovement).Thicken(InCullDistance);
				const FAABB3 MeshQueryBounds = ConvexBounds.InverseTransformedAABB(MeshToConvexTransform);
				const FReal CullDistance = InCullDistance;

				// Collect all the triangles that overlap our convex. Triangles will be in Convex space
				Mesh.CollectTriangles(MeshQueryBounds, MeshToConvexTransform, ConvexBounds, ContactGenerator);

				FContactPointManifold Manifold;

				const auto& GenerateConvexTriangleContacts =
					[&Convex, &ConvexTransform, &ConvexRelativeMovement, CullDistance, &Manifold](Private::FMeshContactGenerator& ContactGenerator, const int32 TriangleIndex)
				{
					Manifold.Reset();

					GenerateConvexTriangleOneShotManifoldMACD(Convex, ConvexTransform, ConvexRelativeMovement, ContactGenerator, TriangleIndex, CullDistance, Manifold);

					ContactGenerator.AddTriangleContacts(TriangleIndex, MakeArrayView(Manifold));
				};

				ContactGenerator.GenerateMeshContacts(GenerateConvexTriangleContacts);

				// Process the contacts to minimize manifold etc
				// MACD does not require further normal fixup
				ContactGenerator.SetFixNormalsEnabled(false);
				ContactGenerator.ProcessGeneratedContacts(ConvexTransform, MeshToConvexTransform);
			}
		}

		/**
		* @brief Create a minimized set of contact points between a convex polyhedron (box, convex) and a non-convex mesh (trimesh, heightfield)
		*
		* @tparam ConvexType any convex type (Sphere, Capsule, Box, Convex, possibly with a scaled or instanced)
		* @tparam MeshType any triangle mesh type (HeightField or TriangleMesh, without a scaled or instanced wrapper)
		* @param MeshQueryBounds Triangles overlapping this box will be tested. Should be in the space of the mesh.
		* @param MeshToConvexTransform The transform from Mesh space to Convex space. This low-level convex-triangle collision detection is performed in Convex space.
		*
		* @see ConstructPlanarConvexTriMeshOneShotManifoldImp, ConstructPlanarConvexHeightfieldOneShotManifoldImp
		*/
		template<typename ConvexType, typename MeshType>
		void GenerateConvexMeshContactPoints(const ConvexType& Convex, const MeshType& Mesh, const FAABB3& MeshQueryBounds, const FRigidTransform3& MeshToConvexTransform, const FReal CullDistance, FContactTriangleCollector& MeshContacts)
		{
			FContactPointManifold TriangleManifoldPoints;

			// Loop over all the triangles, build a manifold and add the points to the total manifold
			// NOTE: contact points will be in the space of the convex until the end of the function when we convert into shape local space
			Mesh.VisitTriangles(MeshQueryBounds, MeshToConvexTransform, [&](const FTriangle& Triangle, const int32 TriangleIndex, const int32 VertexIndex0, const int32 VertexIndex1, const int32 VertexIndex2)
			{
				// Generate the manifold for this triangle
				TriangleManifoldPoints.Reset();
				GenerateConvexTriangleOneShotManifold(Convex, Triangle, CullDistance, TriangleManifoldPoints);

				if (TriangleManifoldPoints.Num() > 0)
				{
					// Add the points into the main contact array
					// NOTE: The Contacts' FaceIndices will be an index into the ContactTriangles not the original tri mesh (this will get mapped back to the mesh index below)
					MeshContacts.AddTriangleContacts(MakeArrayView(TriangleManifoldPoints.begin(), TriangleManifoldPoints.Num()), Triangle, TriangleIndex, VertexIndex0, VertexIndex1, VertexIndex2, CullDistance);
				}
			});

			// Reduce contacts to a minimum manifold and transform contact data back into shape-local space
			MeshContacts.ProcessContacts(MeshToConvexTransform);
		}

		/**
		 * @brief Used by all the convex types to generate a manifold against any mesh type
		*/
		template<typename ConvexType, typename MeshType>
		void ConstructConvexMeshOneShotManifold(const ConvexType& Convex, const FRigidTransform3& ConvexTransform, const MeshType& Mesh, const FRigidTransform3& MeshTransform, const FVec3& MeshScale, const FReal CullDistance, FContactTriangleCollector& MeshContacts)
		{
			FRigidTransform3 MeshToConvexTransform = MeshTransform.GetRelativeTransformNoScale(ConvexTransform);
			MeshToConvexTransform.SetScale3D(MeshScale);

			// @todo(chaos): add Convex.CalculateInverseTransformed bounds with scale support (to optimize sphere and capsule)
			const FAABB3 MeshQueryBounds = Convex.BoundingBox().InverseTransformedAABB(MeshToConvexTransform).Thicken(CullDistance);

			// Create the minimal manifold from all the overlapping triangles
			GenerateConvexMeshContactPoints(Convex, Mesh, MeshQueryBounds, MeshToConvexTransform, CullDistance, MeshContacts);
		}

		void ConstructQuadraticConvexTriMeshOneShotManifold(const FImplicitObject& Quadratic, const FRigidTransform3& QuadraticTransform, const FImplicitObject& InMesh, const FRigidTransform3& MeshTransform, const FReal Dt, FPBDCollisionConstraint& Constraint)
		{
			// All convexes are pre-scaled, or wrapped in TImplicitObjectScaled
			ensure(QuadraticTransform.GetScale3D() == FVec3(1));
			ensure(MeshTransform.GetScale3D() == FVec3(1));

			// Unwrap the tri mesh (remove Scaled or Instanced) and get the scale
			FVec3 MeshScale;
			FReal MeshMargin;	// Not used - will be zero for meshes
			const FTriangleMeshImplicitObject* Mesh = UnwrapImplicit<FTriangleMeshImplicitObject>(InMesh, MeshScale, MeshMargin);
			check(Mesh != nullptr);

			const FReal CullDistance = Constraint.GetCullDistance();

			if (bChaos_Collision_EnableMeshManifoldOptimizedLoop)
			{
				// New version uses a two-pass loop over triangles to avoid visiting triangles whose vertices are all colliding as a result of checking adjacent triangles
				Private::FMeshContactGeneratorSettings ContactGeneratorSettings;
				ContactGeneratorSettings.FaceNormalDotThreshold = 0.9999;	// ~0.8deg Normals must be accurate or rolling will not work correctly
				ContactGeneratorSettings.bUseTwoPassLoop = false;			// two-pass loop is not helpful for capsules and spheres
				ContactGeneratorSettings.bSortByPhi = bChaos_Collision_ConvexTriMeshSortByPhi;
				ContactGeneratorSettings.bSortForSolverConvergence = bChaos_Collision_ConvexTriMeshSortByDistance && !bChaos_Collision_ConvexTriMeshSortByPhi;
				Private::FMeshContactGenerator ContactGenerator(ContactGeneratorSettings);

				const FVec3 RelativeMovement = FVec3(Constraint.GetRelativeMovement());

				if (const FImplicitSphere3* Sphere = Quadratic.template GetObject<FImplicitSphere3>())
				{
					ConstructConvexMeshOneShotManifoldMACD(*Sphere, QuadraticTransform, *Mesh, MeshTransform, MeshScale, RelativeMovement, CullDistance, ContactGenerator);
				}
				else if (const FImplicitCapsule3* Capsule = Quadratic.template GetObject<FImplicitCapsule3>())
				{
					ConstructConvexMeshOneShotManifold2(*Capsule, QuadraticTransform, *Mesh, MeshTransform, MeshScale, CullDistance, ContactGenerator);
				}
				else
				{
					check(false);
				}

				Constraint.SetOneShotManifoldContacts(ContactGenerator.GetContactPoints());
			}
			else
			{
				const FReal PhiTolerance = CalculateTriMeshPhiTolerance(CullDistance);
				const FReal DistanceTolerance = Chaos_Collision_TriMeshDistanceTolerance;
				FContactTriangleCollector MeshContacts(bChaos_Collision_OneSidedTriangleMesh, PhiTolerance, DistanceTolerance, QuadraticTransform);

				if (const FImplicitSphere3* Sphere = Quadratic.template GetObject<FImplicitSphere3>())
				{
					ConstructConvexMeshOneShotManifold(*Sphere, QuadraticTransform, *Mesh, MeshTransform, MeshScale, CullDistance, MeshContacts);
				}
				else if (const FImplicitCapsule3* Capsule = Quadratic.template GetObject<FImplicitCapsule3>())
				{
					ConstructConvexMeshOneShotManifold(*Capsule, QuadraticTransform, *Mesh, MeshTransform, MeshScale, CullDistance, MeshContacts);
				}
				else
				{
					check(false);
				}

				Constraint.SetOneShotManifoldContacts(MeshContacts.GetContactPoints());
			}
		}

		void ConstructQuadraticConvexHeightFieldOneShotManifold(const FImplicitObject& Quadratic, const FRigidTransform3& QuadraticTransform, const FHeightField& Mesh, const FRigidTransform3& MeshTransform, const FReal Dt, FPBDCollisionConstraint& Constraint)
		{
			// We only build one shot manifolds once
			// All convexes are pre-scaled, or wrapped in TImplicitObjectScaled
			ensure(QuadraticTransform.GetScale3D() == FVec3(1));
			ensure(MeshTransform.GetScale3D() == FVec3(1));

			const FVec3 MeshScale = FVec3(1);	// Scale is built into heightfield
			const FReal CullDistance = Constraint.GetCullDistance();

			if (bChaos_Collision_EnableMeshManifoldOptimizedLoop)
			{
				// New version uses a two-pass loop over triangles to avoid visiting triangles whose vertices are all colliding as a result of checking adjacent triangles
				Private::FMeshContactGeneratorSettings ContactGeneratorSettings;
				ContactGeneratorSettings.FaceNormalDotThreshold = 0.9999;	// ~0.8deg Normals must be accurate or rolling will not work correctly
				ContactGeneratorSettings.bUseTwoPassLoop = false;			// two-pass loop is not helpful for capsules and spheres
				ContactGeneratorSettings.bSortByPhi = bChaos_Collision_ConvexTriMeshSortByPhi;
				ContactGeneratorSettings.bSortForSolverConvergence = bChaos_Collision_ConvexTriMeshSortByDistance && !bChaos_Collision_ConvexTriMeshSortByPhi;
				Private::FMeshContactGenerator ContactGenerator(ContactGeneratorSettings);

				if (const FImplicitSphere3* Sphere = Quadratic.template GetObject<FImplicitSphere3>())
				{
					ConstructConvexMeshOneShotManifold2(*Sphere, QuadraticTransform, Mesh, MeshTransform, MeshScale, CullDistance, ContactGenerator);
				}
				else if (const FImplicitCapsule3* Capsule = Quadratic.template GetObject<FImplicitCapsule3>())
				{
					ConstructConvexMeshOneShotManifold2(*Capsule, QuadraticTransform, Mesh, MeshTransform, MeshScale, CullDistance, ContactGenerator);
				}
				else
				{
					check(false);
				}

				Constraint.SetOneShotManifoldContacts(ContactGenerator.GetContactPoints());
			}
			else
			{
				const FReal PhiTolerance = CalculateTriMeshPhiTolerance(CullDistance);
				const FReal DistanceTolerance = Chaos_Collision_TriMeshDistanceTolerance;
				FContactTriangleCollector MeshContacts(bChaos_Collision_OneSidedHeightField, PhiTolerance, DistanceTolerance, QuadraticTransform);

				if (const FImplicitSphere3* Sphere = Quadratic.template GetObject<FImplicitSphere3>())
				{
					ConstructConvexMeshOneShotManifold(*Sphere, QuadraticTransform, Mesh, MeshTransform, MeshScale, CullDistance, MeshContacts);
				}
				else if (const FImplicitCapsule3* Capsule = Quadratic.template GetObject<FImplicitCapsule3>())
				{
					ConstructConvexMeshOneShotManifold(*Capsule, QuadraticTransform, Mesh, MeshTransform, MeshScale, CullDistance, MeshContacts);
				}
				else
				{
					check(false);
				}

				Constraint.SetOneShotManifoldContacts(MeshContacts.GetContactPoints());
			}
		}

		/**
		* @brief Populate the Constraint with a manifold of contacts between a Convex and a TriangleMesh
		* @param Convex A convex polyhedron (Box, Convex) that may be wrapped in Scaled or Instanced
		* @param InMesh A TriangleMesh ImplicitObject that may be wrapped in Scaled or Instaned
		*/
		void ConstructPlanarConvexTriMeshOneShotManifold(const FImplicitObject& Convex, const FRigidTransform3& ConvexTransform, const FImplicitObject& InMesh, const FRigidTransform3& MeshTransform, FPBDCollisionConstraint& Constraint)
		{
			// All convexes are pre-scaled, or wrapped in TImplicitObjectScaled
			ensure(ConvexTransform.GetScale3D() == FVec3(1));
			ensure(MeshTransform.GetScale3D() == FVec3(1));

			// Unwrap the tri mesh (remove Scaled or Instanced) and get the scale
			FVec3 MeshScale;
			FReal MeshMargin;	// Not used - will be zero for meshes
			const FTriangleMeshImplicitObject* Mesh = UnwrapImplicit<FTriangleMeshImplicitObject>(InMesh, MeshScale, MeshMargin);
			check(Mesh != nullptr);

			const FReal CullDistance = Constraint.GetCullDistance();
			const FReal PhiTolerance = CalculateTriMeshPhiTolerance(CullDistance);
			const FReal DistanceTolerance = Chaos_Collision_TriMeshDistanceTolerance;

			if (bChaos_Collision_EnableMeshManifoldOptimizedLoop_TriMesh)
			{
				const FVec3 RelativeMovement = FVec3(Constraint.GetRelativeMovement());
				Private::FMeshContactGeneratorSettings ContactGeneratorSettings;
				ContactGeneratorSettings.bSortByPhi = bChaos_Collision_ConvexTriMeshSortByPhi;
				ContactGeneratorSettings.bSortForSolverConvergence = bChaos_Collision_ConvexTriMeshSortByDistance && !bChaos_Collision_ConvexTriMeshSortByPhi;
				Private::FMeshContactGenerator ContactGenerator(ContactGeneratorSettings);

				if (const FImplicitBox3* RawBox = Convex.template GetObject<FImplicitBox3>())
				{
					ConstructConvexMeshOneShotManifoldMACD(*RawBox, ConvexTransform, *Mesh, MeshTransform, MeshScale, RelativeMovement, CullDistance, ContactGenerator);
				}
				else if (const TImplicitObjectScaled<FImplicitConvex3>* ScaledConvex = Convex.template GetObject<TImplicitObjectScaled<FImplicitConvex3>>())
				{
					ConstructConvexMeshOneShotManifoldMACD(*ScaledConvex, ConvexTransform, *Mesh, MeshTransform, MeshScale, RelativeMovement, CullDistance, ContactGenerator);
				}
				else if (const TImplicitObjectInstanced<FImplicitConvex3>* InstancedConvex = Convex.template GetObject<TImplicitObjectInstanced<FImplicitConvex3>>())
				{
					ConstructConvexMeshOneShotManifoldMACD(*InstancedConvex, ConvexTransform, *Mesh, MeshTransform, MeshScale, RelativeMovement, CullDistance, ContactGenerator);
				}
				else if (const FImplicitConvex3* RawConvex = Convex.template GetObject<FImplicitConvex3>())
				{
					ConstructConvexMeshOneShotManifoldMACD(*RawConvex, ConvexTransform, *Mesh, MeshTransform, MeshScale, RelativeMovement, CullDistance, ContactGenerator);
				}
				else
				{
					check(false);
				}

				Constraint.SetOneShotManifoldContacts(ContactGenerator.GetContactPoints());
			}
			else
			{
				FContactTriangleCollector MeshContacts(bChaos_Collision_OneSidedTriangleMesh, PhiTolerance, DistanceTolerance, ConvexTransform);

				if (const FImplicitBox3* RawBox = Convex.template GetObject<FImplicitBox3>())
				{
					ConstructConvexMeshOneShotManifold(*RawBox, ConvexTransform, *Mesh, MeshTransform, MeshScale, CullDistance, MeshContacts);
				}
				else if (const TImplicitObjectScaled<FImplicitConvex3>* ScaledConvex = Convex.template GetObject<TImplicitObjectScaled<FImplicitConvex3>>())
				{
					ConstructConvexMeshOneShotManifold(*ScaledConvex, ConvexTransform, *Mesh, MeshTransform, MeshScale, CullDistance, MeshContacts);
				}
				else if (const TImplicitObjectInstanced<FImplicitConvex3>* InstancedConvex = Convex.template GetObject<TImplicitObjectInstanced<FImplicitConvex3>>())
				{
					ConstructConvexMeshOneShotManifold(*InstancedConvex, ConvexTransform, *Mesh, MeshTransform, MeshScale, CullDistance, MeshContacts);
				}
				else if (const FImplicitConvex3* RawConvex = Convex.template GetObject<FImplicitConvex3>())
				{
					ConstructConvexMeshOneShotManifold(*RawConvex, ConvexTransform, *Mesh, MeshTransform, MeshScale, CullDistance, MeshContacts);
				}
				else
				{
					check(false);
				}

				Constraint.SetOneShotManifoldContacts(MeshContacts.GetContactPoints());
			}
		}

		/**
		* @brief Populate the Constraint with a manifold of contacts between a Convex and a HeightField
		* @param Convex A convex polyhedron (Box, Convex) that may be wrapped in Scaled or Instanced
		*/
		void ConstructPlanarConvexHeightFieldOneShotManifold(const FImplicitObject& Convex, const FRigidTransform3& ConvexTransform, const FHeightField& Mesh, const FRigidTransform3& MeshTransform, FPBDCollisionConstraint& Constraint)
		{
			// All convexes are pre-scaled, or wrapped in TImplicitObjectScaled
			ensure(ConvexTransform.GetScale3D() == FVec3(1));
			ensure(MeshTransform.GetScale3D() == FVec3(1));

			const FVec3 MeshScale = FVec3(1);	// Scale is built into heightfield
			const FReal CullDistance = Constraint.GetCullDistance();
			const FReal PhiTolerance = CalculateTriMeshPhiTolerance(CullDistance);
			const FReal DistanceTolerance = Chaos_Collision_TriMeshDistanceTolerance;

			if (bChaos_Collision_EnableMeshManifoldOptimizedLoop)
			{
				// New version uses a two-pass loop over triangles to avoid visiting triangles whose vertices are all colliding as a result of checking adjacent triangles
				Private::FMeshContactGeneratorSettings ContactGeneratorSettings;
				ContactGeneratorSettings.bSortByPhi = bChaos_Collision_ConvexTriMeshSortByPhi;
				ContactGeneratorSettings.bSortForSolverConvergence = bChaos_Collision_ConvexTriMeshSortByDistance && !bChaos_Collision_ConvexTriMeshSortByPhi;
				Private::FMeshContactGenerator ContactGenerator(ContactGeneratorSettings);
				const FVec3 RelativeMovement = FVec3(Constraint.GetRelativeMovement());

				if (const FImplicitBox3* RawBox = Convex.template GetObject<FImplicitBox3>())
				{
					ConstructConvexMeshOneShotManifoldMACD(*RawBox, ConvexTransform, Mesh, MeshTransform, MeshScale, RelativeMovement, CullDistance, ContactGenerator);
				}
				else if (const TImplicitObjectScaled<FImplicitConvex3>* ScaledConvex = Convex.template GetObject<TImplicitObjectScaled<FImplicitConvex3>>())
				{
					ConstructConvexMeshOneShotManifoldMACD(*ScaledConvex, ConvexTransform, Mesh, MeshTransform, MeshScale, RelativeMovement, CullDistance, ContactGenerator);
				}
				else if (const TImplicitObjectInstanced<FImplicitConvex3>* InstancedConvex = Convex.template GetObject<TImplicitObjectInstanced<FImplicitConvex3>>())
				{
					ConstructConvexMeshOneShotManifoldMACD(*InstancedConvex, ConvexTransform, Mesh, MeshTransform, MeshScale, RelativeMovement, CullDistance, ContactGenerator);
				}
				else if (const FImplicitConvex3* RawConvex = Convex.template GetObject<FImplicitConvex3>())
				{
					ConstructConvexMeshOneShotManifoldMACD(*RawConvex, ConvexTransform, Mesh, MeshTransform, MeshScale, RelativeMovement, CullDistance, ContactGenerator);
				}
				else
				{
					check(false);
				}

				Constraint.SetOneShotManifoldContacts(ContactGenerator.GetContactPoints());
			}
			else
			{
				FContactTriangleCollector MeshContacts(bChaos_Collision_OneSidedHeightField, PhiTolerance, DistanceTolerance, ConvexTransform);

				if (const FImplicitBox3* RawBox = Convex.template GetObject<FImplicitBox3>())
				{
					ConstructConvexMeshOneShotManifold(*RawBox, ConvexTransform, Mesh, MeshTransform, MeshScale, CullDistance, MeshContacts);
				}
				else if (const TImplicitObjectScaled<FImplicitConvex3>* ScaledConvex = Convex.template GetObject<TImplicitObjectScaled<FImplicitConvex3>>())
				{
					ConstructConvexMeshOneShotManifold(*ScaledConvex, ConvexTransform, Mesh, MeshTransform, MeshScale, CullDistance, MeshContacts);
				}
				else if (const TImplicitObjectInstanced<FImplicitConvex3>* InstancedConvex = Convex.template GetObject<TImplicitObjectInstanced<FImplicitConvex3>>())
				{
					ConstructConvexMeshOneShotManifold(*InstancedConvex, ConvexTransform, Mesh, MeshTransform, MeshScale, CullDistance, MeshContacts);
				}
				else if (const FImplicitConvex3* RawConvex = Convex.template GetObject<FImplicitConvex3>())
				{
					ConstructConvexMeshOneShotManifold(*RawConvex, ConvexTransform, Mesh, MeshTransform, MeshScale, CullDistance, MeshContacts);
				}
				else
				{
					check(false);
				}

				Constraint.SetOneShotManifoldContacts(MeshContacts.GetContactPoints());
			}
		}
	}
}

