// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/Core.h"
#include "Chaos/CollisionResolutionTypes.h"
#include "Chaos/ImplicitFwd.h"

namespace Chaos
{
	class FHeightField;
	class FPBDCollisionConstraint;

	namespace Collisions
	{
		/**
		 * @brief Build the contact manifold between a Quadratic Convex shape and a TriMesh
		 * @param Convex an ImplicitObject that must be a sphere or capsule. Assert if not
		 * @param TriMesh an ImplicitObject that must be FTriangleMeshImplicitObject or a wrapper around it. Asserts if not
		*/
		void ConstructQuadraticConvexTriMeshOneShotManifold(
			const FImplicitObject& Quadratic,
			const FRigidTransform3& Quadraticransform,
			const FImplicitObject& TriangleMesh,
			const FRigidTransform3& TriMeshTransform,
			const FReal Dt,
			FPBDCollisionConstraint& Constraint);

		/**
		 * @brief Build the contact manifold between a Quadratic Convex shape (Sphere, Capsule) and a TriMesh
		 * @param Convex an ImplicitObject that must be a sphere or capsule. Assert if not
		*/
		void ConstructQuadraticConvexHeightFieldOneShotManifold(
			const FImplicitObject& Quadratic,
			const FRigidTransform3& QuadraticTransform,
			const FHeightField& HeightField,
			const FRigidTransform3& HeightFieldTransform,
			const FReal Dt,
			FPBDCollisionConstraint& Constraint);

		/**
		 * @brief Build the contact manifold between a Planar Convex shape and a TriMesh
		 * @param Convex an ImplicitObject that must be a box, convex, or a wrapper around it. Asserts if not
		 * @param TriMesh an ImplicitObject that must be FTriangleMeshImplicitObject or a wrapper around it. Asserts if not
		*/
		void ConstructPlanarConvexTriMeshOneShotManifold(
			const FImplicitObject& Convex, 
			const FRigidTransform3& ConvexTransform, 
			const FImplicitObject& TriangleMesh, 
			const FRigidTransform3& TriangleMeshTransform, 
			FPBDCollisionConstraint& Constraint);

		/**
		 * @brief Build the contact manifold between a Planar Convex shape and a HeightField
		 * @param Convex an ImplicitObject that must be a box, convex, or a wrapper around it. Asserts if not
		*/
		void ConstructPlanarConvexHeightFieldOneShotManifold(
			const FImplicitObject& Convex, 
			const FRigidTransform3& ConvexTransform, 
			const FHeightField& HeightField, 
			const FRigidTransform3& TriangleMeshTransform, 
			FPBDCollisionConstraint& Constraint);

	}
}