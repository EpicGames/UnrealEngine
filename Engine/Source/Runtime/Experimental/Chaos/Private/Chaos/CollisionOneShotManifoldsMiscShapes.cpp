// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/CollisionOneShotManifoldsMiscShapes.h"

#include "Chaos/CollisionOneShotManifolds.h"
#include "Chaos/CollisionResolution.h"
#include "Chaos/Collision/CapsuleConvexContactPoint.h"
#include "Chaos/Collision/ContactPointsMiscShapes.h"
#include "Chaos/Collision/SphereConvexContactPoint.h"
#include "Chaos/Collision/PBDCollisionConstraint.h"
#include "Chaos/Convex.h"
#include "Chaos/Defines.h"
#include "Chaos/Framework/UncheckedArray.h"
#include "Chaos/GJK.h"
#include "Chaos/ImplicitObjectScaled.h"
#include "Chaos/Transform.h"
#include "Chaos/Utilities.h"
#include "ChaosStats.h"

//PRAGMA_DISABLE_OPTIMIZATION

namespace Chaos
{
	extern FRealSingle Chaos_Collision_Manifold_SphereCapsuleSizeThreshold;
	extern FRealSingle Chaos_Collision_Manifold_CapsuleAxisAlignedThreshold;
	extern FRealSingle Chaos_Collision_Manifold_CapsuleDeepPenetrationFraction;
	extern FRealSingle Chaos_Collision_Manifold_CapsuleRadialContactFraction;

	namespace Collisions
	{

		void ConstructSphereSphereOneShotManifold(
			const FSphere& SphereA,
			const FRigidTransform3& SphereATransform, //world
			const FSphere& SphereB,
			const FRigidTransform3& SphereBTransform, //world
			const FReal Dt,
			FPBDCollisionConstraint& Constraint)
		{
			SCOPE_CYCLE_COUNTER_MANIFOLD();
			
			// We only build one shot manifolds once
			// All convexes are pre-scaled, or wrapped in TImplicitObjectScaled
			ensure(SphereATransform.GetScale3D() == FVec3(1.0f, 1.0f, 1.0f));
			ensure(SphereBTransform.GetScale3D() == FVec3(1.0f, 1.0f, 1.0f));

			// @todo(chaos): support manifold maintenance
			Constraint.ResetActiveManifoldContacts();

			FContactPoint ContactPoint = SphereSphereContactPoint(SphereA, SphereATransform, SphereB, SphereBTransform, Constraint.GetCullDistancef());
			if (ContactPoint.Phi < Constraint.GetCullDistance())
			{
				Constraint.AddOneshotManifoldContact(ContactPoint);
			}
		}

		void ConstructSpherePlaneOneShotManifold(
			const FSphere& Sphere, 
			const FRigidTransform3& SphereTransform, 
			const TPlane<FReal, 3>& Plane, 
			const FRigidTransform3& PlaneTransform, 
			const FReal Dt, 
			FPBDCollisionConstraint& Constraint)
		{
			// We only build one shot manifolds once
			// All convexes are pre-scaled, or wrapped in TImplicitObjectScaled
			ensure(SphereTransform.GetScale3D() == FVec3(1.0f, 1.0f, 1.0f));
			ensure(PlaneTransform.GetScale3D() == FVec3(1.0f, 1.0f, 1.0f));

			// @todo(chaos): support manifold maintenance
			Constraint.ResetActiveManifoldContacts();

			FContactPoint ContactPoint = SpherePlaneContactPoint(Sphere, SphereTransform, Plane, PlaneTransform);
			if (ContactPoint.Phi < Constraint.GetCullDistance())
			{
				Constraint.AddOneshotManifoldContact(ContactPoint);
			}
		}	

		void ConstructSphereBoxOneShotManifold(const FSphere& Sphere, const FRigidTransform3& SphereTransform, const FImplicitBox3& Box, const FRigidTransform3& BoxTransform, const FReal Dt, FPBDCollisionConstraint& Constraint)
		{
			// We only build one shot manifolds once
			// All convexes are pre-scaled, or wrapped in TImplicitObjectScaled
			ensure(SphereTransform.GetScale3D() == FVec3(1.0f, 1.0f, 1.0f));
			ensure(BoxTransform.GetScale3D() == FVec3(1.0f, 1.0f, 1.0f));

			// @todo(chaos): support manifold maintenance
			Constraint.ResetActiveManifoldContacts();

			FContactPoint ContactPoint = SphereBoxContactPoint(Sphere, SphereTransform, Box, BoxTransform);
			if (ContactPoint.Phi < Constraint.GetCullDistance())
			{
				Constraint.AddOneshotManifoldContact(ContactPoint);
			}
		}

		// Build a sphere-Capsule manifold.
		// When the sphere and capsule are of similar size, we usually only need a 1-point manifold.
		// If the sphere is larger than the capsule, we need to generate a multi-point manifold so that
		// we don't end up jittering between collisions on each end cap. E.g., consider a small capsule
		// lying horizontally on a very large sphere (almost flat) - we need at least 2 contact points to 
		// make this stable.
		void ConstructSphereCapsuleOneShotManifold(const FSphere& Sphere, const FRigidTransform3& SphereTransform, const FCapsule& Capsule, const FRigidTransform3& CapsuleTransform, const FReal Dt, FPBDCollisionConstraint& Constraint)
		{
			// We only build one shot manifolds once
			// All convexes are pre-scaled, or wrapped in TImplicitObjectScaled
			ensure(SphereTransform.GetScale3D() == FVec3(1.0f, 1.0f, 1.0f));
			ensure(CapsuleTransform.GetScale3D() == FVec3(1.0f, 1.0f, 1.0f));

			Constraint.ResetActiveManifoldContacts();

			// Build a multi-point manifold
			const FRealSingle NetCullDistance = Sphere.GetRadiusf() + Capsule.GetRadiusf() + Constraint.GetCullDistancef();
			const FRealSingle NetCullDistanceSq = FMath::Square(NetCullDistance);

			// Transform the sphere into capsule space and find the closest point on the capsule line segment
			// @todo(chaos) this would be much simpler if the sphere's were always at the origin and capsules were at the origin and axis aligned
			const FRigidTransform3f SphereToCapsuleTransform = FRigidTransform3f(SphereTransform.GetRelativeTransformNoScale(CapsuleTransform));
			const FVec3f SpherePos = SphereToCapsuleTransform.TransformPositionNoScale(Sphere.GetCenterf());
			const FRealSingle NearPosT = Utilities::ClosestTimeOnLineSegment<FRealSingle>(SpherePos, Capsule.GetX1f(), Capsule.GetX2f());

			// Add the closest contact point to the manifold
			const FVec3f NearPos = FMath::Lerp(Capsule.GetX1f(), Capsule.GetX2f(), NearPosT);
			const FVec3f NearPosDelta = SpherePos - NearPos;
			const FRealSingle NearPosDistanceSq = NearPosDelta.SizeSquared();
			if (NearPosDistanceSq > UE_SMALL_NUMBER)
			{
				if (NearPosDistanceSq < NetCullDistanceSq)
				{
					const FRealSingle NearPosDistance = FMath::Sqrt(NearPosDistanceSq);
					const FVec3 NearPosDir = NearPosDelta / NearPosDistance;
					const FRealSingle NearPhi = NearPosDistance - Sphere.GetRadiusf() - Capsule.GetRadiusf();

					FContactPointf NearContactPoint;
					NearContactPoint.ShapeContactPoints[0] = SphereToCapsuleTransform.InverseTransformPositionNoScale(SpherePos - Sphere.GetRadiusf() * NearPosDir);
					NearContactPoint.ShapeContactPoints[1] = NearPos + Capsule.GetRadiusf() * NearPosDir;
					NearContactPoint.ShapeContactNormal = NearPosDir;
					NearContactPoint.Phi = NearPhi;
					NearContactPoint.FaceIndex = INDEX_NONE;
					NearContactPoint.ContactType = EContactPointType::VertexPlane;
					Constraint.AddOneshotManifoldContact(NearContactPoint);

					// If we have a small sphere, just stick with the 1-point manifold
					const FRealSingle SphereCapsuleSizeThreshold = Chaos_Collision_Manifold_SphereCapsuleSizeThreshold;
					if (Sphere.GetRadiusf() < SphereCapsuleSizeThreshold * (Capsule.GetHeightf() + Capsule.GetRadiusf()))
					{
						return;
					}

					// If the capsule is non-dynamic there's no point in creating the multipoint manifold
					if (!FConstGenericParticleHandle(Constraint.GetParticle1())->IsDynamic())
					{
						return;
					}

					// If the contact is deep, there's a high chance that pushing one end out will push the other deeper and we also need more contacts.
					// Note: we only consider the radius of the dynamic object(s) when deciding what "deep" means because the extra contacts are only
					// to prevent excessive rotation from the single contact we have so far, and only the dynamic objects will rotate.
					const FRealSingle DeepRadiusFraction = Chaos_Collision_Manifold_CapsuleDeepPenetrationFraction;
					const bool bIsDeep = NearPhi < -DeepRadiusFraction * Capsule.GetRadiusf();
					if (!bIsDeep)
					{
						return;
					}

					// Now add the two end caps
					// Calculate the vector orthogonal to the capsule axis that gives the nearest points on the capsule cyclinder to the sphere
					// The initial length will be proportional to the sine of the angle between the axis and the delta position and will approach
					// zero when the capsule is end-on to the sphere, in which case we won't add the end caps.
					constexpr FRealSingle EndCapSinAngleThreshold = 0.35f;	// about 20deg
					constexpr FRealSingle EndCapDistanceThreshold = 0.2f;	// fraction
					FVec3f CapsuleOrthogonal = FVec3f::CrossProduct(Capsule.GetAxisf(), FVec3::CrossProduct(Capsule.GetAxisf(), NearPosDir));
					const FRealSingle CapsuleOrthogonalLenSq = CapsuleOrthogonal.SizeSquared();
					if (CapsuleOrthogonalLenSq > FMath::Square(EndCapSinAngleThreshold))
					{
						// Orthogonal must point towards the sphere, but currently depends on the relative axis orientation
						CapsuleOrthogonal = CapsuleOrthogonal * FMath::InvSqrt(CapsuleOrthogonalLenSq);
						if (FVec3f::DotProduct(CapsuleOrthogonal, SpherePos - Capsule.GetCenterf()) < 0.0f)
						{
							CapsuleOrthogonal = -CapsuleOrthogonal;
						}

						if (NearPosT > EndCapDistanceThreshold)
						{
							const FVec3f EndCapPos0 = Capsule.GetX1f() + CapsuleOrthogonal * Capsule.GetRadiusf();
							const FRealSingle EndCapDistance0 = (SpherePos - EndCapPos0).Size();
							const FRealSingle EndCapPhi0 = EndCapDistance0 - Sphere.GetRadiusf();
							
							if (EndCapPhi0 < Constraint.GetCullDistance())
							{
								const FVec3f EndCapPosDir0 = (SpherePos - EndCapPos0) / EndCapDistance0;
								const FVec3f SpherePos0 = SpherePos - EndCapPosDir0 * Sphere.GetRadiusf();
						
								FContactPointf EndCapContactPoint0;
								EndCapContactPoint0.ShapeContactPoints[0] = SphereToCapsuleTransform.InverseTransformPositionNoScale(SpherePos0);
								EndCapContactPoint0.ShapeContactPoints[1] = EndCapPos0;
								EndCapContactPoint0.ShapeContactNormal = EndCapPosDir0;
								EndCapContactPoint0.Phi = EndCapPhi0;
								EndCapContactPoint0.FaceIndex = INDEX_NONE;
								EndCapContactPoint0.ContactType = EContactPointType::VertexPlane;
								Constraint.AddOneshotManifoldContact(EndCapContactPoint0);
							}
						}

						if (NearPosT < 1.0f - EndCapDistanceThreshold)
						{
							const FVec3f EndCapPos1 = Capsule.GetX2f() + CapsuleOrthogonal * Capsule.GetRadiusf();
							const FRealSingle EndCapDistance1 = (SpherePos - EndCapPos1).Size();
							const FRealSingle EndCapPhi1 = EndCapDistance1 - Sphere.GetRadiusf();

							if (EndCapPhi1 < Constraint.GetCullDistance())
							{
								const FVec3f EndCapPosDir1 = (SpherePos - EndCapPos1) / EndCapDistance1;
								const FVec3f SpherePos1 = SpherePos - EndCapPosDir1 * Sphere.GetRadiusf();

								FContactPointf EndCapContactPoint0;
								EndCapContactPoint0.ShapeContactPoints[0] = SphereToCapsuleTransform.InverseTransformPositionNoScale(SpherePos1);
								EndCapContactPoint0.ShapeContactPoints[1] = EndCapPos1;
								EndCapContactPoint0.ShapeContactNormal = EndCapPosDir1;
								EndCapContactPoint0.Phi = EndCapPhi1;
								EndCapContactPoint0.FaceIndex = INDEX_NONE;
								EndCapContactPoint0.ContactType = EContactPointType::VertexPlane;
								Constraint.AddOneshotManifoldContact(EndCapContactPoint0);
							}
						}
					}
				}
			}
		}

		template<typename ConvexType>
		void ConstructSphereConvexManifoldImpl(const FImplicitSphere3& Sphere, const ConvexType& Convex, const FRigidTransform3& SphereToConvexTransform, const FReal CullDistance, FContactPointManifold& ContactPoints)
		{
			FContactPoint ClosestContactPoint = SphereConvexContactPoint(Sphere, Convex, SphereToConvexTransform);

			// Stop now if beyond cull distance
			if (ClosestContactPoint.Phi > CullDistance)
			{
				return;
			}

			// We always use the primary contact so add it to the output now
			ContactPoints.Add(ClosestContactPoint);

			// If the sphere is "large" compared to the convex add more points
			const FVec3 SpherePos = SphereToConvexTransform.TransformPositionNoScale(FVec3(Sphere.GetCenterf()));
			const FReal SphereRadius = Sphere.GetRadiusf();
			const FReal SpheerConvexManifoldSizeThreshold = FReal(1);
			const FReal ConvexSize = Convex.BoundingBox().Extents().GetAbsMax();
			if (SphereRadius > SpheerConvexManifoldSizeThreshold * ConvexSize)
			{
				// Find the convex plane to use - the one most opposing the primary contact normal
				const int32 ConvexPlaneIndex = Convex.GetMostOpposingPlane(-ClosestContactPoint.ShapeContactNormal);
				if (ConvexPlaneIndex != INDEX_NONE)
				{
					FVec3 ConvexPlanePosition, ConvexPlaneNormal;
					Convex.GetPlaneNX(ConvexPlaneIndex, ConvexPlaneNormal, ConvexPlanePosition);

					// Project the face verts onto the sphere along the normal and generate speculative contacts
					// We actually just take a third of the points, chosen arbitrarily. This may not be the best choice for convexes where
					// most of the face verts are close to each other with a few outliers. 
					// @todo(chaos): a better option would be to build a triangle of contacts around the primary contact, with the verts projected into the convex face
					const int32 NumConvexPlaneVertices = Convex.NumPlaneVertices(ConvexPlaneIndex);
					const int32 PlaneVertexStride = FMath::Max(1, NumConvexPlaneVertices / 3);
					for (int32 PlaneVertexIndex = 0; PlaneVertexIndex < NumConvexPlaneVertices; PlaneVertexIndex += PlaneVertexStride)
					{
						const FVec3 ConvexPlaneVertex = Convex.GetVertex(Convex.GetPlaneVertex(ConvexPlaneIndex, PlaneVertexIndex));
						const FReal ConvexContactDistance = Utilities::RaySphereIntersectionDistance(ConvexPlaneVertex, ClosestContactPoint.ShapeContactNormal, SpherePos, SphereRadius);
						if (ConvexContactDistance < CullDistance)
						{
							FContactPoint& ConvexContactPoint = ContactPoints[ContactPoints.AddUninitialized()];
							ConvexContactPoint.ShapeContactPoints[0] = SphereToConvexTransform.InverseTransformPositionNoScale(ConvexPlaneVertex + ClosestContactPoint.ShapeContactNormal * ConvexContactDistance);
							ConvexContactPoint.ShapeContactPoints[1] = ConvexPlaneVertex;
							ConvexContactPoint.ShapeContactNormal = ClosestContactPoint.ShapeContactNormal;
							ConvexContactPoint.Phi = ConvexContactDistance;
							ConvexContactPoint.FaceIndex = INDEX_NONE;
							ConvexContactPoint.ContactType = EContactPointType::VertexPlane;

							if (ContactPoints.IsFull())
							{
								break;
							}
						}
					}
				}
			}
		}

		void ConstructSphereConvexManifold(const FSphere& Sphere, const FRigidTransform3& SphereTransform, const FImplicitObject3& Convex, const FRigidTransform3& ConvexTransform, const FReal Dt, FPBDCollisionConstraint& Constraint)
		{
			// We only build one shot manifolds once
			// All convexes are pre-scaled, or wrapped in TImplicitObjectScaled
			ensure(SphereTransform.GetScale3D() == FVec3(1.0f, 1.0f, 1.0f));
			ensure(ConvexTransform.GetScale3D() == FVec3(1.0f, 1.0f, 1.0f));

			const FRigidTransform3 SphereToConvexTransform = SphereTransform.GetRelativeTransformNoScale(ConvexTransform);

			FContactPointManifold ContactPoints;
			if (const FImplicitBox3* RawBox = Convex.template GetObject<FImplicitBox3>())
			{
				ConstructSphereConvexManifoldImpl(Sphere, *RawBox, SphereToConvexTransform, Constraint.GetCullDistance(), ContactPoints);
			}
			else if (const TImplicitObjectScaled<FImplicitConvex3>* ScaledConvex = Convex.template GetObject<TImplicitObjectScaled<FImplicitConvex3>>())
			{
				ConstructSphereConvexManifoldImpl(Sphere, *ScaledConvex, SphereToConvexTransform, Constraint.GetCullDistance(), ContactPoints);
			}
			else if (const TImplicitObjectInstanced<FImplicitConvex3>* InstancedConvex = Convex.template GetObject<TImplicitObjectInstanced<FImplicitConvex3>>())
			{
				ConstructSphereConvexManifoldImpl(Sphere, *InstancedConvex, SphereToConvexTransform, Constraint.GetCullDistance(), ContactPoints);
			}
			else if (const FImplicitConvex3* RawConvex = Convex.template GetObject<FImplicitConvex3>())
			{
				ConstructSphereConvexManifoldImpl(Sphere, *RawConvex, SphereToConvexTransform, Constraint.GetCullDistance(), ContactPoints);
			}
			else
			{
				check(false);
			}

			// Add the points to the constraint
			Constraint.ResetActiveManifoldContacts();
			for (FContactPoint& ContactPoint : ContactPoints)
			{
				Constraint.AddOneshotManifoldContact(ContactPoint);
			}
		}

		void ConstructCapsuleCapsuleOneShotManifold(const FCapsule& CapsuleA, const FRigidTransform3& CapsuleATransform, const FCapsule& CapsuleB, const FRigidTransform3& CapsuleBTransform, const FReal Dt, FPBDCollisionConstraint& Constraint)
		{
			// We only build one shot manifolds once
			// All convexes are pre-scaled, or wrapped in TImplicitObjectScaled
			ensure(CapsuleATransform.GetScale3D() == FVec3(1.0f, 1.0f, 1.0f));
			ensure(CapsuleBTransform.GetScale3D() == FVec3(1.0f, 1.0f, 1.0f));

			// @todo(chaos): support manifold maintenance
			Constraint.ResetActiveManifoldContacts();


			const FRigidTransform3f CapsuleAToCapsuleBf = FRigidTransform3f(CapsuleATransform.GetRelativeTransformNoScale(CapsuleBTransform));

			FVec3f AAxis(CapsuleAToCapsuleBf.TransformVector(CapsuleA.GetAxisf()));
			const FVec3f BAxis = CapsuleB.GetAxisf();

			const FRealSingle AHalfLen = CapsuleA.GetHeightf() / 2.0f;
			const FRealSingle BHalfLen = CapsuleB.GetHeightf() / 2.0f;

			// Used in a few places below where we need to use the smaller/larger capsule, but always a dynamic one
			const FRealSingle ADynamicRadius = FConstGenericParticleHandle(Constraint.GetParticle0())->IsDynamic() ? CapsuleA.GetRadiusf() : TNumericLimits<FRealSingle>::Max();
			const FRealSingle BDynamicRadius = FConstGenericParticleHandle(Constraint.GetParticle1())->IsDynamic() ? CapsuleB.GetRadiusf() : TNumericLimits<FRealSingle>::Max();

			// Make both capsules point in the same general direction
			FRealSingle ADotB = FVec3f::DotProduct(AAxis, BAxis);
			if (ADotB < 0)
			{
				ADotB = -ADotB;
				AAxis = -AAxis;
			}

			// Get the closest points on the two line segments. This is used to generate the closest contact point
			// which is always added to the manifold (if within CullDistance). We may also add other points.
			FVector3f AClosest, BClosest;
			const FVector3f ACenter = CapsuleAToCapsuleBf.TransformPosition(CapsuleA.GetCenterf());
			const FVector3f BCenter = CapsuleB.GetCenterf();
			FMath::SegmentDistToSegmentSafe(
				ACenter + AHalfLen * AAxis,
				ACenter - AHalfLen * AAxis,
				BCenter + BHalfLen * BAxis,
				BCenter - BHalfLen * BAxis,
				AClosest,
				BClosest);

			FVec3f ClosestDelta = BClosest - AClosest;
			FRealSingle ClosestDeltaLen = ClosestDelta.Size();

			// Stop now if we are beyond the cull distance
			const FRealSingle ClosestPhi = ClosestDeltaLen - (CapsuleA.GetRadiusf() + CapsuleB.GetRadiusf());
			if (ClosestPhi > Constraint.GetCullDistance())
			{
				return;
			}

			// Calculate the normal from the two closest points. Handle exact axis overlaps.
			FVec3f ClosestNormal;
			if (ClosestDeltaLen > UE_KINDA_SMALL_NUMBER)
			{
				ClosestNormal = -ClosestDelta / ClosestDeltaLen;
			}
			else
			{
				// Center axes exactly intersect. We'll fake a result that pops the capsules out along the Z axis, with the smaller capsule going up
				ClosestNormal = (ADynamicRadius <= BDynamicRadius) ? FVec3f(0, 0, 1) : FVec3f(0, 0, -1);
			}
			const FVec3f ClosestLocationA = AClosest - ClosestNormal * CapsuleA.GetRadiusf();
			const FVec3f ClosestLocationB = BClosest + ClosestNormal * CapsuleB.GetRadiusf();

			// We always add the closest point to the manifold
			// We may also add 2 more points generated from the end cap positions of the smaller capsule
			FContactPointf ClosestContactPoint;
			ClosestContactPoint.ShapeContactPoints[0] = CapsuleAToCapsuleBf.InverseTransformPositionNoScale(ClosestLocationA);
			ClosestContactPoint.ShapeContactPoints[1] = ClosestLocationB;
			ClosestContactPoint.ShapeContactNormal = ClosestNormal;
			ClosestContactPoint.Phi = ClosestPhi;
			ClosestContactPoint.FaceIndex = INDEX_NONE;
			ClosestContactPoint.ContactType = EContactPointType::VertexPlane;
			Constraint.AddOneshotManifoldContact(ClosestContactPoint);

			// We don't generate manifold points within this fraction (of segment length) distance
			constexpr FRealSingle TDeltaThreshold = 0.2f;		// fraction

			// If the nearest cylinder normal is parallel to the other axis within this tolerance, we stick with 1 manifold point
			constexpr FRealSingle SinAngleThreshold = 0.35f;	// about 20deg (this would be an endcap-versus-cylinderwall collision at >70 degs)

			// If the capsules are in an X configuration, this controls the distance of the manifold points from the closest point
			const FRealSingle RadialContactFraction = Chaos_Collision_Manifold_CapsuleRadialContactFraction;

			// Calculate the line segment times for the nearest point calculate above
			// NOTE: TA and TB will be in [-1, 1]
			const FRealSingle TA = FVec3f::DotProduct(AClosest - ACenter, AAxis) / AHalfLen;
			const FRealSingle TB = FVec3f::DotProduct(BClosest - BCenter, BAxis) / BHalfLen;

			// If we have an end-end contact with no segment overlap, stick with the single point manifold
			// This is when we have two capsules laid end to end (as opposed to side-by-side)
			// NOTE: This test only works because we made the axes point in the same direction above
			if ((TA < FRealSingle(-1) + TDeltaThreshold) && (TB > FRealSingle(1) - TDeltaThreshold))
			{
				return;
			}
			if ((TB < FRealSingle(-1) + TDeltaThreshold) && (TA > FRealSingle(1) - TDeltaThreshold))
			{
				return;
			}

			// If the axes are closely aligned, we definitely want more contact points (e.g., capsule lying on top of another).
			// Also if the contact is deep, there's a high chance that pushing one end out will push the other deeper and we also need more contacts.
			// Note: we only consider the radius of the dynamic object(s) when deciding what "deep" means because the extra contacts are only
			// to prevent excessive rotation from the single contact we have so far, and only the dynamic objects will rotate.
			const FRealSingle AxisDotMinimum = Chaos_Collision_Manifold_CapsuleAxisAlignedThreshold;
			const FRealSingle DeepRadiusFraction = Chaos_Collision_Manifold_CapsuleDeepPenetrationFraction;
			const FRealSingle MinDynamicRadius = FMath::Min(ADynamicRadius, BDynamicRadius);
			const bool bAreAligned = ADotB > AxisDotMinimum;
			const bool bIsDeep = ClosestPhi < -DeepRadiusFraction * MinDynamicRadius;
			if (!bAreAligned && !bIsDeep)
			{
				return;
			}

			// Lambda: Create a contact point between a point on the cylinder of FirstCapsule at FirstT, with the nearest point on SecondCapsule
			auto MakeCapsuleSegmentContact = [](
				const FRealSingle FirstT,
				const FVec3f& FirstCenter,
				const FVec3f& FirstAxis,
				const FRealSingle FirstHalfLen,
				const FRealSingle FirstRadius,
				const FRigidTransform3f& FirstTransformToSecond,
				const FVec3f& SecondCenter,
				const FVec3f& SecondAxis,
				const FRealSingle SecondHalfLen,
				const FRealSingle SecondRadius,
				const FVec3f& Orthogonal,
				const FRealSingle CullDistance,
				const bool bSwap) -> FContactPointf
			{
				FContactPointf ContactPoint;

				const FVec3f FirstContactPos = FirstCenter + (FirstT * FirstHalfLen) * FirstAxis + Orthogonal * FirstRadius;
				const FVec3f SecondSegmentPos = FMath::ClosestPointOnLine(SecondCenter - SecondHalfLen * SecondAxis, SecondCenter + SecondHalfLen * SecondAxis, FirstContactPos);
				const FRealSingle SecondSegmentDist = (FirstContactPos - SecondSegmentPos).Size();
				const FVec3f SecondSegmentDir = (FirstContactPos - SecondSegmentPos) / SecondSegmentDist;
				const FVec3f SecondContactPos = SecondSegmentPos + SecondRadius * SecondSegmentDir;
				const FRealSingle ContactPhi = SecondSegmentDist - SecondRadius;

				if (ContactPhi < CullDistance)
				{
					if (!bSwap)
					{
						ContactPoint.ShapeContactPoints[0] = FirstTransformToSecond.InverseTransformPositionNoScale(FirstContactPos);
						ContactPoint.ShapeContactPoints[1] = SecondContactPos;
						ContactPoint.ShapeContactNormal = SecondSegmentDir;
					}
					else
					{
						ContactPoint.ShapeContactPoints[0] = FirstTransformToSecond.InverseTransformPositionNoScale(SecondContactPos);
						ContactPoint.ShapeContactPoints[1] = FirstContactPos;
						ContactPoint.ShapeContactNormal = -SecondSegmentDir;
					}
					ContactPoint.Phi = ContactPhi;
					ContactPoint.FaceIndex = INDEX_NONE;
					ContactPoint.ContactType = EContactPointType::VertexPlane;
				}

				return ContactPoint;
			};

			// Lambda: Add up to 2 more contacts from the cylindrical surface on FirstCylinder, if they are not too close to the existing contact.
			// The point locations depend on cylinder alignment.
			auto MakeCapsuleEndPointContacts = [&Constraint, TDeltaThreshold, SinAngleThreshold, RadialContactFraction, &MakeCapsuleSegmentContact](
				const FRealSingle FirstT,
				const FVec3f& FirstCenter,
				const FVec3f& FirstAxis,
				const FRealSingle FirstHalfLen,
				const FRealSingle FirstRadius,
				const FRigidTransform3f& FirstTransformToSecond,
				const FVec3f& SecondCenter,
				const FVec3f& SecondAxis,
				const FRealSingle SecondHalfLen,
				const FRealSingle SecondRadius,
				const FVec3f& ClosestDir,
				const FRealSingle FirstAxisDotSecondAxis,
				const bool bSwap) -> void
			{
				// Orthogonal: the vector from a point on FirstCapsule's axis to its cylinder surface, in the direction of SecondCapsule
				FVec3f Orthogonal = FVec3f::CrossProduct(FirstAxis, FVec3f::CrossProduct(FirstAxis, ClosestDir));
				const FRealSingle OrthogonalLenSq = Orthogonal.SizeSquared();
				if (OrthogonalLenSq > FMath::Square(SinAngleThreshold))
				{
					Orthogonal = Orthogonal * FMath::InvSqrt(OrthogonalLenSq);
					if (FVec3f::DotProduct(Orthogonal, SecondCenter - FirstCenter) < FRealSingle(0))
					{
						Orthogonal = -Orthogonal;
					}

					// Clip the FirstCapsule's end points to be within the line segment of SecondCapsule
					// This is to restrict the extra contacts to the overlapping line segment (e.g, when capsules are lying partly on top of each other)
					const FRealSingle ProjectedLen = FRealSingle(2) * FirstHalfLen * FirstAxisDotSecondAxis;
					const FRealSingle ClippedTMin = FVec3f::DotProduct((SecondCenter - SecondHalfLen * SecondAxis) - (FirstCenter + FirstHalfLen * FirstAxis), SecondAxis) / ProjectedLen;
					const FRealSingle ClippedTMax = FVec3f::DotProduct((SecondCenter + SecondHalfLen * SecondAxis) - (FirstCenter - FirstHalfLen * FirstAxis), SecondAxis) / ProjectedLen;

					// Clip the FirstCapsules end points to be within some laterial distance od the SecondCapsule's axis
					// This restricts the contacts to be at a useful location when line segments are perpendicular to each other 
					// (e.g., when the capsules are on top of each other but in a cross)
					// As we get more perpendicular, move limits closer to radius fraction
					const FRealSingle MaxDeltaTRadial = RadialContactFraction * (SecondRadius / FirstHalfLen);
					const FRealSingle RadialClippedTMax = FMath::Lerp(FRealSingle(MaxDeltaTRadial), FRealSingle(1), FirstAxisDotSecondAxis);

					const FRealSingle TMin = FMath::Max3(FRealSingle(-1), ClippedTMin, -RadialClippedTMax);
					const FRealSingle TMax = FMath::Min3(FRealSingle(1), ClippedTMax, RadialClippedTMax);

					if (TMin < FirstT - TDeltaThreshold)
					{
						FContactPointf EndContact0 = MakeCapsuleSegmentContact(TMin, FirstCenter, FirstAxis, FirstHalfLen, FirstRadius, FirstTransformToSecond, SecondCenter, SecondAxis, SecondHalfLen, SecondRadius, Orthogonal, Constraint.GetCullDistancef(), bSwap);
						if (EndContact0.Phi < Constraint.GetCullDistance())
						{
							Constraint.AddOneshotManifoldContact(EndContact0);
						}
					}
					if (TMax > FirstT + TDeltaThreshold)
					{
						FContactPointf EndContact1 = MakeCapsuleSegmentContact(TMax, FirstCenter, FirstAxis, FirstHalfLen, FirstRadius, FirstTransformToSecond, SecondCenter, SecondAxis, SecondHalfLen, SecondRadius, Orthogonal, Constraint.GetCullDistancef(), bSwap);
						if (EndContact1.Phi < Constraint.GetCullDistance())
						{
							Constraint.AddOneshotManifoldContact(EndContact1);
						}
					}
				}
			};

			// Generate the extra manifold points
			if (ADynamicRadius <= BDynamicRadius)
			{
				MakeCapsuleEndPointContacts(TA, ACenter, AAxis, AHalfLen, CapsuleA.GetRadiusf(), CapsuleAToCapsuleBf, BCenter, BAxis, BHalfLen, CapsuleB.GetRadiusf(), ClosestNormal, ADotB, false);
			}
			else
			{
				MakeCapsuleEndPointContacts(TB, BCenter, BAxis, BHalfLen, CapsuleB.GetRadiusf(), CapsuleAToCapsuleBf, ACenter, AAxis, AHalfLen, CapsuleA.GetRadiusf(), ClosestNormal, ADotB, true);
			}
		}

	} // namespace Collisions
} // namespace Chaos

