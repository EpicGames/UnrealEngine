// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/Core.h"
#include "Chaos/CollisionResolutionTypes.h"
#include "Chaos/ImplicitFwd.h"

namespace Chaos
{
	class FHeightField;
	class FPBDCollisionConstraint;

	namespace Collisions
	{
		void ConstructSphereSphereOneShotManifold(
			const FSphere& Sphere1,
			const FRigidTransform3& Convex1Transform,
			const FSphere& Sphere2,
			const FRigidTransform3& Convex2Transform,
			const FReal Dt,
			FPBDCollisionConstraint& Constraint);

		void ConstructSpherePlaneOneShotManifold(
			const FSphere& Sphere,
			const FRigidTransform3& SphereTransform,
			const TPlane<FReal, 3>& Plane,
			const FRigidTransform3& PlaneTransform,
			const FReal Dt,
			FPBDCollisionConstraint& Constraint);

		void ConstructSphereBoxOneShotManifold(
			const FSphere& Sphere,
			const FRigidTransform3& SphereTransform,
			const FImplicitBox3& Box,
			const FRigidTransform3& BoxTransform,
			const FReal Dt,
			FPBDCollisionConstraint& Constraint);

		void ConstructSphereCapsuleOneShotManifold(
			const FSphere& Sphere,
			const FRigidTransform3& SphereTransform,
			const FCapsule& Capsule,
			const FRigidTransform3& CapsuleTransform,
			const FReal Dt,
			FPBDCollisionConstraint& Constraint);

		void ConstructSphereConvexManifold(
			const FSphere& Sphere,
			const FRigidTransform3& SphereTransform,
			const FImplicitObject3& Convex,
			const FRigidTransform3& ConvexTransform,
			const FReal Dt,
			FPBDCollisionConstraint& Constraint);

		void ConstructCapsuleCapsuleOneShotManifold(
			const FCapsule& CapsuleA,
			const FRigidTransform3& CapsuleATransform,
			const FCapsule& CapsuleB,
			const FRigidTransform3& CapsuleBTransform,
			const FReal Dt,
			FPBDCollisionConstraint& Constraint);
	}
}