// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/CollisionResolution.h"

#include "Chaos/ChaosPerfTest.h"
#include "Chaos/Capsule.h"
#include "Chaos/CCDUtilities.h"
#include "Chaos/CollisionOneShotManifolds.h"
#include "Chaos/CollisionOneShotManifoldsMeshShapes.h"
#include "Chaos/CollisionOneShotManifoldsMiscShapes.h"
#include "Chaos/CollisionResolutionTypes.h"
#include "Chaos/CollisionResolutionUtil.h"
#include "Chaos/Collision/CapsuleConvexContactPoint.h"
#include "Chaos/Collision/CollisionContext.h"
#include "Chaos/Collision/CollisionConstraintAllocator.h"
#include "Chaos/Collision/CollisionFilter.h"
#include "Chaos/Collision/CollisionUtil.h"
#include "Chaos/Collision/ContactPointsMiscShapes.h"
#include "Chaos/Collision/GJKContactPointSwept.h"
#include "Chaos/Collision/GJKContactPoint.h"
#include "Chaos/Collision/PBDCollisionConstraint.h"
#include "Chaos/Collision/SphereConvexContactPoint.h"
#include "Chaos/Convex.h"
#include "Chaos/Defines.h"
#include "Chaos/GeometryQueries.h"
#include "Chaos/HeightField.h"
#include "Chaos/ImplicitFwd.h"
#include "Chaos/ImplicitObjectScaled.h"
#include "Chaos/ImplicitObjectTransformed.h"
#include "Chaos/Levelset.h"
#include "Chaos/Pair.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/Particle/ParticleUtilities.h"
#include "Chaos/PBDCollisionConstraints.h"
#include "Chaos/Sphere.h"
#include "Chaos/Transform.h"
#include "Chaos/TriangleMeshImplicitObject.h"

//UE_DISABLE_OPTIMIZATION


// @todo(chaos): clean up the contact creation time rejection to avoid extra transforms
#define CHAOS_COLLISION_CREATE_BOUNDSCHECK 1

DECLARE_CYCLE_STAT(TEXT("Collisions::UpdateGenericConvexConvexConstraint"), STAT_Collisions_UpdateGenericConvexConvexConstraint, STATGROUP_ChaosCollision);
DECLARE_CYCLE_STAT(TEXT("Collisions::UpdateConvexTriangleMeshConstraint"), STAT_Collisions_UpdateConvexTriangleMeshConstraint, STATGROUP_ChaosCollision);
DECLARE_CYCLE_STAT(TEXT("Collisions::UpdateConvexTriangleMeshConstraintSwept"), STAT_Collisions_UpdateConvexTriangleMeshConstraintSwept, STATGROUP_ChaosCollision);
DECLARE_CYCLE_STAT(TEXT("Collisions::UpdateConvexHeightFieldConstraint"), STAT_Collisions_UpdateConvexHeightFieldConstraint, STATGROUP_ChaosCollision);
DECLARE_CYCLE_STAT(TEXT("Collisions::UpdateGenericConvexConvexConstraintSwept"), STAT_Collisions_UpdateGenericConvexConvexConstraintSwept, STATGROUP_ChaosCollision);
DECLARE_CYCLE_STAT(TEXT("Collisions::UpdateCapsuleHeightFieldConstraintSwept"), STAT_Collisions_UpdateCapsuleHeightFieldConstraintSwept, STATGROUP_ChaosCollision);
DECLARE_CYCLE_STAT(TEXT("Collisions::UpdateCapsuleHeightFieldConstraint"), STAT_Collisions_UpdateCapsuleHeightFieldConstraint, STATGROUP_ChaosCollision);
DECLARE_CYCLE_STAT(TEXT("Collisions::UpdateCapsuleTriangleMeshConstraint"), STAT_Collisions_UpdateCapsuleTriangleMeshConstraint, STATGROUP_ChaosCollision);
DECLARE_CYCLE_STAT(TEXT("Collisions::UpdateCapsuleTriangleMeshConstraintSwept"), STAT_Collisions_UpdateCapsuleTriangleMeshConstraintSwept, STATGROUP_ChaosCollision);
DECLARE_CYCLE_STAT(TEXT("Collisions::UpdateConvexHeightFieldConstraintSwept"), STAT_Collisions_UpdateConvexHeightFieldConstraintSwept, STATGROUP_ChaosCollision);
DECLARE_CYCLE_STAT(TEXT("Collisions::UpdateLevelsetLevelsetConstraint"), STAT_UpdateLevelsetLevelsetConstraint, STATGROUP_ChaosCollision);

DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("NumContactUpdates"), STAT_ChaosCollisionCounter_NumContactUpdates, STATGROUP_ChaosCollisionCounters);

int32 CCDUseGenericSweptConvexConstraints = 1;
FAutoConsoleVariableRef CVarUseGenericSweptConvexConstraints(TEXT("p.Chaos.CCD.UseGenericSweptConvexConstraints"), CCDUseGenericSweptConvexConstraints, TEXT("Use generic convex convex swept constraint generation for convex shape pairs which don't have specialized implementations."));

int32 CCDOnlyConsiderDynamicStatic = 0;
FAutoConsoleVariableRef CVarCCDOnlyConsiderDynamicStatic(TEXT("p.Chaos.CCD.OnlyConsiderDynamicStatic"), CCDOnlyConsiderDynamicStatic, TEXT("Only enable CCD for dynamic-static pairs."));

int32 ConstraintsDetailedStats = 0;
FAutoConsoleVariableRef CVarConstraintsDetailedStats(TEXT("p.Chaos.Constraints.DetailedStats"), ConstraintsDetailedStats, TEXT("When set to 1, will enable more detailed stats."));

bool bChaos_Collision_AllowLevelsetManifolds = true;
FAutoConsoleVariableRef CVarChaosCollisionAllowLevelsetManifolds(TEXT("p.Chaos.Collision.AllowLevelsetManifolds"), bChaos_Collision_AllowLevelsetManifolds, TEXT("Use incremental manifolds for levelset-levelset collision. This does not work well atm - too much rotation in the small pieces"));

bool CCDNoCullAllShapePairs = true;
FAutoConsoleVariableRef CVarCCDNoCullAllShapePairs(TEXT("p.Chaos.CCD.NoCullAllShapePairs"), CCDNoCullAllShapePairs, TEXT("Whether to cull contacts early based on phi for sweeps for all shape pairs (not just convex convex)."));

bool Chaos_Collision_NarrowPhase_SphereBoundsCheck = true;
bool Chaos_Collision_NarrowPhase_AABBBoundsCheck = true;
FAutoConsoleVariableRef CVarChaosCollisionSphereBoundsCheck(TEXT("p.Chaos.Collision.SphereBoundsCheck"), Chaos_Collision_NarrowPhase_SphereBoundsCheck, TEXT(""));
FAutoConsoleVariableRef CVarChaosCollisionAABBBoundsCheck(TEXT("p.Chaos.Collision.AABBBoundsCheck"), Chaos_Collision_NarrowPhase_AABBBoundsCheck, TEXT(""));

bool bChaos_Collision_ShapesArrayMode = true;
FAutoConsoleVariableRef CVarChaos_Collision_ShapesArrayMode(TEXT("p.Chaos.Collision.ShapesArrayMode"), bChaos_Collision_ShapesArrayMode, TEXT(""));

bool bChaosCCDResetTOIFix = true;
FAutoConsoleVariableRef CVarChaos_Collision_ResetTOIFix(TEXT("p.Chaos.Bugs.ResetTOIFix"), bChaosCCDResetTOIFix, TEXT(""));

namespace Chaos
{
	namespace Collisions
	{

		// Traits to control how contacts are generated
		template<bool B_IMMEDIATEUPDATE>
		struct TConstructCollisionTraits
		{
			// If true, ConstructConstraints also initializes the constraint Phi and Normal based on current state, and
			// contacts beyond CullDistance are culled. If false, a constraint is created for every shape pair.
			// NOTE: Contact Phi and Normal are also calculated at the beginning of each iteration as well. The reason you may
			// want bImmediateUpdate=true is to reduce the number of contacts that make it to the constraint graph (because
			// we have culled those with a separation greater than CullDistance).
			// However, early culling can cause collisions to be missed, e.g., if a joint moves a body into a space where
			// a culled collision should now be active. This can (does) lead to jitter in some cases.
			static const bool bImmediateUpdate = B_IMMEDIATEUPDATE;
		};

		/** Determines if body should use CCD. If using CCD, computes Dir and Length of sweep.
		* If either of the object is CCD-enabled and fast moving, we enable CCD for the constraint.
		* We compare P-X and BoundingBox().Extents().Min() to determine if an object is fast-moving or not. Here is how this works:
		* Think about a box colliding with a static plane. If P-X <= 0.5 * BoundingBox().Extents().Min(), it is impossible for the box to tunnel through the plane. Otherwise, it is possible but not necessary to have tunneling.
		* Now think about two boxes. If P-X <= 0.5 * BoundingBox().Extents().Min() both boxes, it is impossible for one box to tunnel through the other (it is still possible for a box's corner to move through another box's corner).
		* So far this is sufficient for us to determine CCD constraints for our primitives. But the caveat is that for arbitrary objects (think about a rod that is not axis aligned), min of a bounding box extent can be too large and tunneling could still happen if P-X falls below this threshold.
		*/
		bool ShouldUseCCD(const FGeometryParticleHandle* Particle0, const FVec3& DeltaX0, const FGeometryParticleHandle* Particle1, const FVec3& DeltaX1, FVec3& Dir, FReal& Length)
		{
			if (CCDOnlyConsiderDynamicStatic > 0 && Particle1->ObjectState() != EObjectStateType::Static)
			{
				return false;
			}

			const TPBDRigidParticleHandle<FReal, 3>* Rigid0 = Particle0->CastToRigidParticle();
			const bool bIsCCDEnabled0 = Rigid0 && Rigid0->CCDEnabled();
			const TPBDRigidParticleHandle<FReal, 3>* Rigid1 = Particle1->CastToRigidParticle();
			const bool bIsCCDEnabled1 = Rigid1 && Rigid1->CCDEnabled();
			if (!bIsCCDEnabled0 && !bIsCCDEnabled1)
			{
				return false;
			}

			bool bUseCCD = FCCDHelpers::DeltaExceedsThreshold(
				Particle0->CCDAxisThreshold(), DeltaX0, FConstGenericParticleHandle(Particle0)->Q(),
				Particle1->CCDAxisThreshold(), DeltaX1, FConstGenericParticleHandle(Particle1)->Q());

			if (bUseCCD)
			{
				Dir = DeltaX0 - DeltaX1;
				Length = Dir.SafeNormalize();
				if (Length < UE_SMALL_NUMBER) // This is the case where both particles' velocities are high but the relative velocity is low. 
				{
					bUseCCD = false;
				}

				// Do not perform CCD if the vector is not close to unit length to prevent getting caught in a large or infinite loop when raycasting.
				// @todo(chaos): What's this? Dir is normalized above so this seems unnecessary unless it is checking for 0 length, not unit length - remove it or fix
				if (!FMath::IsNearlyEqual((float)Dir.SizeSquared(), 1.f, UE_KINDA_SMALL_NUMBER))
				{
					bUseCCD = false;
				}
			}

			return bUseCCD;
		}

		// Add the contact to the manifold (or update the existing point). Disable the contact if the contact distance is greater than the Cull Distance
		void UpdateContactPoint(FPBDCollisionConstraint& Constraint, const FContactPoint& ContactPoint, const FReal Dt)
		{
			// Permanently disable contacts beyond the CullDistance
			// NOTE: We cannot do this for incremental manifolds because of a hack in the heightfield collision detection.
			// See FHeightField::GJKContactPointImp where it uses a sweep.
#if CHAOS_COLLISION_CREATE_BOUNDSCHECK
			const bool bIsIncrementalManifold = Constraint.GetUseManifold() && (Constraint.GetManifoldPoints().Num() > 0);
			if ((ContactPoint.Phi > Constraint.GetCullDistance()) && !bIsIncrementalManifold)
			{
				Constraint.SetDisabled(true);
				return;
			}
#endif

			// Ignore points that have not been initialized - i.e., if there is no detectable contact 
			// point within reasonable range despite passing the AABB tests
			if (ContactPoint.IsSet())
			{
				Constraint.AddIncrementalManifoldContact(ContactPoint);
			}
			else
			{
				// If we did not add a new contact point, select the best existing one
				// We only hit this because of the FHeightField::GJKContactPointImp hack mentioned above
				// @todo(chaos): fix heightfield collision and get rid of this
				if (Constraint.GetUseManifold())
				{
					Constraint.UpdateManifoldContacts();
				}
			}
		}

		// Same as UpdateContact Point but without checking CullDistance. Used by CCD because sweeps do not set the separation unless the sweep actually hits
		void UpdateContactPointNoCull(FPBDCollisionConstraint& Constraint, const FContactPoint& ContactPoint, const FReal Dt, const bool bNoCull = true)
		{
			if (!bNoCull)
			{
				if (ContactPoint.Phi > Constraint.GetCullDistance())
				{
					Constraint.SetDisabled(true);
					return;
				}
			}

			if (ContactPoint.IsSet())
			{
				Constraint.AddIncrementalManifoldContact(ContactPoint);
			}
		}

		// This is pretty unnecessary - all instanced shapes have the same implementation so we should be able to
		// collapse this switch into a generic call. Maybe add a base class to TImplicitObjectInstanced.
		inline const FImplicitObject* GetInstancedImplicit(const FImplicitObject* Implicit0)
		{
			EImplicitObjectType Implicit0OuterType = Implicit0->GetType();

			if (Implicit0OuterType == TImplicitObjectInstanced<FConvex>::StaticType())
			{
				return Implicit0->template GetObject<const TImplicitObjectInstanced<FConvex>>()->GetInstancedObject();
			}
			else if (Implicit0OuterType == TImplicitObjectInstanced<TBox<FReal, 3>>::StaticType())
			{
				return Implicit0->template GetObject<const TImplicitObjectInstanced<TBox<FReal, 3>>>()->GetInstancedObject();
			}
			else if (Implicit0OuterType == TImplicitObjectInstanced<FCapsule>::StaticType())
			{
				return Implicit0->template GetObject<const TImplicitObjectInstanced<FCapsule>>()->GetInstancedObject();
			}
			else if (Implicit0OuterType == TImplicitObjectInstanced<FSphere>::StaticType())
			{
				return Implicit0->template GetObject<const TImplicitObjectInstanced<FSphere>>()->GetInstancedObject();
			}
			else if (Implicit0OuterType == TImplicitObjectInstanced<FConvex>::StaticType())
			{
				return Implicit0->template GetObject<const TImplicitObjectInstanced<FConvex>>()->GetInstancedObject();
			}
			else if (Implicit0OuterType == TImplicitObjectInstanced<FTriangleMeshImplicitObject>::StaticType())
			{
				return Implicit0->template GetObject<const TImplicitObjectInstanced<FTriangleMeshImplicitObject>>()->GetInstancedObject();
			}

			return nullptr;
		}


		template<typename T_IMPLICITA>
		struct TConvexImplicitTraits
		{
			static const bool bSupportsOneShotManifold = false;
		};

		template<>
		struct TConvexImplicitTraits<FImplicitBox3>
		{
			static const bool bSupportsOneShotManifold = true;
		};
		template<>
		struct TConvexImplicitTraits<FImplicitConvex3>
		{
			static const bool bSupportsOneShotManifold = true;
		};
		template<>
		struct TConvexImplicitTraits<TImplicitObjectInstanced<FImplicitConvex3>>
		{
			static const bool bSupportsOneShotManifold = true;
		};
		template<>
		struct TConvexImplicitTraits<TImplicitObjectScaled<FImplicitConvex3>>
		{
			static const bool bSupportsOneShotManifold = true;
		};

		// Convex pair type traits
		template<typename T_IMPLICITA, typename T_IMPLICITB>
		struct TConvexImplicitPairTraits
		{
			// Whether the pair types should use a manifold
			static const bool bSupportsOneShotManifold = TConvexImplicitTraits<T_IMPLICITA>::bSupportsOneShotManifold && TConvexImplicitTraits<T_IMPLICITB>::bSupportsOneShotManifold;
		};

		// Convex pair type traits
		template<typename T_IMPLICITB>
		struct TCapsuleImplicitPairTraits
		{
			// Whether the Capsules can use the generic path. If false, a specialized path exists
			static const bool bSupportsGenericOneShotManifoldPath = false;
		};

		// Convex pair type traits
		template<>
		struct TCapsuleImplicitPairTraits<FConvex>
		{
			static const bool bSupportsGenericOneShotManifoldPath = true;
		};

		// Convex pair type traits
		template<>
		struct TCapsuleImplicitPairTraits<TImplicitObjectScaled<FConvex, 1>>
		{
			static const bool bSupportsGenericOneShotManifoldPath = true;
		};

		// Convex pair type traits
		template<>
		struct TCapsuleImplicitPairTraits<class TImplicitObjectInstanced<FConvex>>
		{
			static const bool bSupportsGenericOneShotManifoldPath = true;
		};

		// Convex pair type traits		

		// Use the traits to call the appropriate convex-convex update method. 
		// Either incremental manifold (default) or one-shot manifold
		template<bool T_SUPPORTSONESHOTMANIFOLD = false>
		struct TConvexConvexUpdater
		{
			template<typename T_ImplicitA, typename T_ImplicitB>
			static void UpdateConvexConvexConstraint(const T_ImplicitA& A, const FRigidTransform3& ATM, const T_ImplicitB& B, const FRigidTransform3& BTM, const FReal Dt, FPBDCollisionConstraint& Constraint)
			{
				UpdateContactPoint(Constraint, GJKContactPoint(A, ATM, B, BTM, FVec3(1, 0, 0)), Dt);
			}
		};

		// One-shot manifold convex-convex update
		template<>
		struct TConvexConvexUpdater<true>
		{
			template<typename T_ImplicitA, typename T_ImplicitB>
			static void UpdateConvexConvexConstraint(const T_ImplicitA& A, const FRigidTransform3& ATM, const T_ImplicitB& B, const FRigidTransform3& BTM, const FReal Dt, FPBDCollisionConstraint& Constraint)
			{
				if (Constraint.GetUseManifold())
				{
					ConstructConvexConvexOneShotManifold(A, ATM, B, BTM, Dt, Constraint);
				}
				else
				{
					UpdateContactPoint(Constraint, GJKContactPoint(A, ATM, B, BTM, FVec3(1, 0, 0)), Dt);
				}
			}
		};

		// Another helper required by UpdateGenericConvexConvexConstraintHelper which uses CastHelper and does not have any typedefs 
		// for the concrete implicit types, so we need to rely on type deduction from the compiler.
		struct FConvexConvexUpdaterCaller
		{
			template<typename T_ImplicitA, typename T_ImplicitB>
			static void Update(const T_ImplicitA& A, const FRigidTransform3& ATM, const T_ImplicitB& B, const FRigidTransform3& BTM, const FReal Dt, FPBDCollisionConstraint& Constraint)
			{
				using FConvexConvexUpdater = TConvexConvexUpdater<TConvexImplicitPairTraits<T_ImplicitA, T_ImplicitB>::bSupportsOneShotManifold>;
				FConvexConvexUpdater::UpdateConvexConvexConstraint(A, ATM, B, BTM, Dt, Constraint);
			}
		};


		// Unwrap the many convex types, including scaled, and call the appropriate update which depends on the concrete types
		void UpdateGenericConvexConvexConstraintHelper(const FImplicitObject& A, const FRigidTransform3& ATM, const FImplicitObject& B, const FRigidTransform3& BTM, const FReal Dt, FPBDCollisionConstraint& Constraint)
		{
			// This expands to a switch of switches that calls the inner function with the appropriate concrete implicit types
			Utilities::CastHelperNoUnwrap(A, ATM, [&](const auto& ADowncast, const FRigidTransform3& AFullTM)
			{
				Utilities::CastHelperNoUnwrap(B, BTM, [&](const auto& BDowncast, const FRigidTransform3& BFullTM)
				{
					FConvexConvexUpdaterCaller::Update(ADowncast, AFullTM, BDowncast, BFullTM, Dt, Constraint);
				});
			});
		}


		template <typename TPGeometryClass>
		const TPGeometryClass* GetInnerObject(const FImplicitObject& Geometry)
		{
			if (const TImplicitObjectScaled<TPGeometryClass>* ScaledConvexImplicit = Geometry.template GetObject<const TImplicitObjectScaled<TPGeometryClass> >())
				return (Geometry.template GetObject<const TImplicitObjectScaled<TPGeometryClass> >())->GetUnscaledObject();
			else if (const TImplicitObjectInstanced<TPGeometryClass>* InstancedImplicit = Geometry.template GetObject<const TImplicitObjectInstanced<TPGeometryClass> >())
				return (Geometry.template GetObject<const TImplicitObjectInstanced<TPGeometryClass> >())->GetInstancedObject();
			else if (const TPGeometryClass* ConvexImplicit = Geometry.template GetObject<const TPGeometryClass>())
				return Geometry.template GetObject<const TPGeometryClass>();
			return nullptr;
		}


		//
		// Box - Box
		//

		FContactPoint BoxBoxContactPoint(const FImplicitBox3& Box1, const FImplicitBox3& Box2, const FRigidTransform3& Box1TM, const FRigidTransform3& Box2TM)
		{
			return GJKContactPoint(Box1, Box1TM, Box2, Box2TM, FVec3(1, 0, 0));
		}

		void UpdateBoxBoxConstraint(const FImplicitBox3& Box1, const FRigidTransform3& Box1Transform, const FImplicitBox3& Box2, const FRigidTransform3& Box2Transform, const FReal Dt, FPBDCollisionConstraint& Constraint)
		{
			if (Constraint.GetUseManifold())
			{
				ConstructBoxBoxOneShotManifold(Box1, Box1Transform, Box2, Box2Transform, Dt, Constraint);
			}
			else
			{
				UpdateContactPoint(Constraint, BoxBoxContactPoint(Box1, Box2, Box1Transform, Box2Transform), Dt);
			}
		}

		//
		// Box - HeightField
		//


		void UpdateBoxHeightFieldConstraint(const FImplicitBox3& A, const FRigidTransform3& ATransform, const FHeightField& B, const FRigidTransform3& BTransform, const FReal Dt, FPBDCollisionConstraint& Constraint)
		{
			if(Constraint.GetUseManifold())
			{
				ConstructPlanarConvexHeightFieldOneShotManifold(A, ATransform, B, BTransform, Constraint);
			}
			else
			{
				UpdateContactPoint(Constraint, BoxHeightFieldContactPoint(A, ATransform, B, BTransform, Constraint.GetCullDistance()), Dt);
			}
		}

		//
		// Box-Plane
		//

		FContactPoint BoxPlaneContactPoint(const FImplicitBox3& Box, const FImplicitPlane3& Plane, const FRigidTransform3& BoxTransform, const FRigidTransform3& PlaneTransform)
		{
			FContactPoint Contact;

#if USING_CODE_ANALYSIS
			MSVC_PRAGMA(warning(push))
			MSVC_PRAGMA(warning(disable : ALL_CODE_ANALYSIS_WARNINGS))
#endif	// USING_CODE_ANALYSIS

			const FRigidTransform3 BoxToPlaneTransform(BoxTransform.GetRelativeTransform(PlaneTransform));
			const FVec3 Extents = Box.Extents();
			constexpr int32 NumCorners = 2 + 2 * 3;
			constexpr FReal Epsilon = UE_KINDA_SMALL_NUMBER;

			FVec3 Corners[NumCorners];
			int32 CornerIdx = 0;
			Corners[CornerIdx++] = BoxToPlaneTransform.TransformPosition(Box.Max());
			Corners[CornerIdx++] = BoxToPlaneTransform.TransformPosition(Box.Min());
			for (int32 j = 0; j < 3; ++j)
			{
				Corners[CornerIdx++] = BoxToPlaneTransform.TransformPosition(Box.Min() + FVec3::AxisVector(j) * Extents);
				Corners[CornerIdx++] = BoxToPlaneTransform.TransformPosition(Box.Max() - FVec3::AxisVector(j) * Extents);
			}

#if USING_CODE_ANALYSIS
			MSVC_PRAGMA(warning(pop))
#endif	// USING_CODE_ANALYSIS

			FVec3 PotentialConstraints[NumCorners];
			int32 NumConstraints = 0;
			for (int32 i = 0; i < NumCorners; ++i)
			{
				FVec3 Normal;
				const FReal NewPhi = Plane.PhiWithNormal(Corners[i], Normal);
				if (NewPhi < Contact.Phi + Epsilon)
				{
					if (NewPhi <= Contact.Phi - Epsilon)
					{
						NumConstraints = 0;
					}
					Contact.Phi = NewPhi;
					Contact.ShapeContactPoints[1] = Corners[i];
					Contact.ShapeContactNormal = Normal;
					PotentialConstraints[NumConstraints++] = Corners[i];
				}
			}
			if (NumConstraints > 1)
			{
				FVec3 AverageLocation(0);
				for (int32 ConstraintIdx = 0; ConstraintIdx < NumConstraints; ++ConstraintIdx)
				{
					AverageLocation += PotentialConstraints[ConstraintIdx];
				}
				Contact.ShapeContactPoints[1] = AverageLocation / static_cast<FReal>(NumConstraints);
			}

			Contact.ShapeContactPoints[0] = BoxToPlaneTransform.InverseTransformPositionNoScale(Contact.ShapeContactPoints[1]);
			Contact.ShapeContactPoints[1] = Contact.ShapeContactPoints[1] - Contact.Phi * Contact.ShapeContactNormal;

			return Contact;
		}

		void UpdateBoxPlaneConstraint(const FImplicitBox3& A, const FRigidTransform3& ATransform, const TPlane<FReal, 3>& B, const FRigidTransform3& BTransform, const FReal Dt, FPBDCollisionConstraint& Constraint)
		{
			// @todo(chaos): one-shot manifold
			UpdateContactPoint(Constraint, BoxPlaneContactPoint(A, B, ATransform, BTransform), Dt);
		}

		//
		// Box-TriangleMesh
		//

		template <typename TriMeshType>
		void UpdateBoxTriangleMeshConstraint(const FImplicitBox3& Box0, const FRigidTransform3& WorldTransform0, const TriMeshType& TriangleMesh1, const FRigidTransform3& WorldTransform1, const FReal Dt, FPBDCollisionConstraint& Constraint)
		{
			if(Constraint.GetUseManifold())
			{
				ConstructPlanarConvexTriMeshOneShotManifold(Box0, WorldTransform0, TriangleMesh1, WorldTransform1, Constraint);
			}
			else
			{
				// @toto(chaos): restitutionpadding
				UpdateContactPoint(Constraint, BoxTriangleMeshContactPoint(Box0, WorldTransform0, TriangleMesh1, WorldTransform1, Constraint.GetCullDistance()), Dt);
			}
		}

		//
		// Sphere - Sphere
		//

		void UpdateSphereSphereConstraint(const FSphere& Sphere1, const FRigidTransform3& Sphere1Transform, const FSphere& Sphere2, const FRigidTransform3& Sphere2Transform, const FReal Dt, FPBDCollisionConstraint& Constraint)
		{
			if (Constraint.GetUseManifold())
			{
				ConstructSphereSphereOneShotManifold(Sphere1, Sphere1Transform, Sphere2, Sphere2Transform, Dt, Constraint);
			}
			else
			{
				UpdateContactPoint(Constraint, SphereSphereContactPoint(Sphere1, Sphere1Transform, Sphere2, Sphere2Transform, Constraint.GetCullDistancef()), Dt);
			}
		}

		//
		// Sphere - HeightField
		//

		void UpdateSphereHeightFieldConstraint(const FSphere& A, const FRigidTransform3& ATransform, const FHeightField& B, const FRigidTransform3& BTransform, const FReal Dt, FPBDCollisionConstraint& Constraint)
		{
			if (Constraint.GetUseManifold())
			{
				ConstructQuadraticConvexHeightFieldOneShotManifold(A, ATransform, B, BTransform, Dt, Constraint);
			}
			else
			{
				UpdateContactPoint(Constraint, SphereHeightFieldContactPoint(A, ATransform, B, BTransform, Constraint.GetCullDistance()), Dt);
			}
		}

		void UpdateSphereHeightFieldConstraintSwept(TGeometryParticleHandle<FReal, 3>* Particle0, const FSphere& A, const FRigidTransform3& ATransform, const FHeightField& B, const FRigidTransform3& BTransform, const FVec3& Dir, const FReal Length, const FReal Dt, FPBDCollisionConstraint& Constraint)
		{
			FReal TOI = FPBDCollisionConstraint::MaxTOI;
			UpdateContactPointNoCull(Constraint, GJKImplicitSweptContactPoint(A, ATransform, B, BTransform, Dir, Length, Constraint.GetCCDEnablePenetration(), Constraint.GetCCDTargetPenetration(), TOI), Dt, CCDNoCullAllShapePairs);
			Constraint.SetCCDTimeOfImpact(TOI);
		}

		//
		//  Sphere-Plane
		//

		void UpdateSpherePlaneConstraint(const FSphere& Sphere, const FRigidTransform3& SphereTransform, const TPlane<FReal, 3>& Plane, const FRigidTransform3& PlaneTransform, const FReal Dt, FPBDCollisionConstraint& Constraint)
		{
			if (Constraint.GetUseManifold())
			{
				ConstructSpherePlaneOneShotManifold(Sphere, SphereTransform, Plane, PlaneTransform, Dt, Constraint);
			}
			else
			{
				UpdateContactPoint(Constraint, SpherePlaneContactPoint(Sphere, SphereTransform, Plane, PlaneTransform), Dt);
			}
		}

		//
		// Sphere - Box
		//


		void UpdateSphereBoxConstraint(const FSphere& Sphere, const FRigidTransform3& SphereTransform, const FImplicitBox3& Box, const FRigidTransform3& BoxTransform, const FReal Dt, FPBDCollisionConstraint& Constraint)
		{
			if (Constraint.GetUseManifold())
			{
				ConstructSphereBoxOneShotManifold(Sphere, SphereTransform, Box, BoxTransform, Dt, Constraint);
			}
			else
			{
				UpdateContactPoint(Constraint, SphereBoxContactPoint(Sphere, SphereTransform, Box, BoxTransform), Dt);
			}
		}

		//
		// Sphere - Capsule
		//


		void UpdateSphereCapsuleConstraint(const FSphere& A, const FRigidTransform3& ATransform, const FCapsule& B, const FRigidTransform3& BTransform, const FReal Dt, FPBDCollisionConstraint& Constraint)
		{
			if (Constraint.GetUseManifold())
			{
				ConstructSphereCapsuleOneShotManifold(A, ATransform, B, BTransform, Dt, Constraint);
			}
			else
			{
				UpdateContactPoint(Constraint, SphereCapsuleContactPoint(A, ATransform, B, BTransform), Dt);
			}
		}

		//
		// Sphere-Convex
		//

		void UpdateSphereConvexConstraint(const FSphere& A, const FRigidTransform3& ATransform, const FImplicitObject3& B, const FRigidTransform3& BTransform, const FReal Dt, FPBDCollisionConstraint& Constraint)
		{
			if (Constraint.GetUseManifold())
			{
				ConstructSphereConvexManifold(A, ATransform, B, BTransform, Dt, Constraint);
			}
			else
			{
				UpdateContactPoint(Constraint, SphereConvexContactPoint(A, ATransform, B, BTransform), Dt);
			}
		}

		//
		// Sphere-TriangleMesh
		//

		template <typename TriMeshType>
		void UpdateSphereTriangleMeshConstraint(const FSphere& Sphere0, const FRigidTransform3& WorldTransform0, const TriMeshType& TriangleMesh1, const FRigidTransform3& WorldTransform1, const FReal Dt, FPBDCollisionConstraint& Constraint)
		{
			if (Constraint.GetUseManifold())
			{
				ConstructQuadraticConvexTriMeshOneShotManifold(Sphere0, WorldTransform0, TriangleMesh1, WorldTransform1, Dt, Constraint);
			}
			else
			{
				// @todo(chaos): restitutionpadding
				UpdateContactPoint(Constraint, SphereTriangleMeshContactPoint(Sphere0, WorldTransform0, TriangleMesh1, WorldTransform1, Constraint.GetCullDistance()), Dt);
			}
		}

		template<typename TriMeshType>
		void UpdateSphereTriangleMeshConstraintSwept(TGeometryParticleHandle<FReal, 3>* Particle0, const FSphere& Sphere0, const FRigidTransform3& WorldTransform0, const TriMeshType& TriangleMesh1, const FRigidTransform3& WorldTransform1, const FVec3& Dir, const FReal Length, const FReal Dt, FPBDCollisionConstraint& Constraint)
		{
			FReal TOI = FPBDCollisionConstraint::MaxTOI;
			UpdateContactPointNoCull(Constraint, SphereTriangleMeshSweptContactPoint(Sphere0, WorldTransform0, TriangleMesh1, WorldTransform1, Dir, Length, Constraint.GetCCDEnablePenetration(), Constraint.GetCCDTargetPenetration(), TOI), Dt, CCDNoCullAllShapePairs);
			Constraint.SetCCDTimeOfImpact(TOI);
		}

		//
		// Capsule-Capsule
		//

		void UpdateCapsuleCapsuleConstraint(const FCapsule& A, const FRigidTransform3& ATransform, const FCapsule& B, const FRigidTransform3& BTransform, const FReal Dt, FPBDCollisionConstraint& Constraint)
		{
			if (Constraint.GetUseManifold())
			{
				ConstructCapsuleCapsuleOneShotManifold(A, ATransform, B, BTransform, Dt, Constraint);
			}
			else
			{
				UpdateContactPoint(Constraint, CapsuleCapsuleContactPoint(A, ATransform, B, BTransform), Dt);
			}
		}

		//
		// Capsule - Box
		//

		void UpdateCapsuleBoxConstraint(const FCapsule& A, const FRigidTransform3& ATransform, const FImplicitBox3& B, const FRigidTransform3& BTransform, const FReal Dt, FPBDCollisionConstraint& Constraint)
		{
			if (Constraint.GetUseManifold())
			{
				FContactPointManifold ContactPoints;
				ConstructCapsuleConvexOneShotManifold<FImplicitBox3>(A, ATransform, B, BTransform, Constraint.GetCullDistance(), ContactPoints);

				Constraint.ResetActiveManifoldContacts();
				for (const FContactPoint& ContactPoint : ContactPoints)
				{
					Constraint.AddOneshotManifoldContact(ContactPoint);
				}
			}
			else
			{
				const FVec3 InitialDir = FVec3(1,0,0);
				UpdateContactPoint(Constraint, CapsuleBoxContactPoint(A, ATransform, B, BTransform, InitialDir), Dt);
			}
		}

		//
		// Capsule-Convex
		//

		// Use the traits to call the appropriate convex-convex update method. 
		// Either incremental manifold (default) or one-shot manifold
		template<bool T_SUPPORTSONESHOTMANIFOLD = false>
		struct TCapsuleConvexUpdater
		{
			template<typename T_ImplicitB>
			static void UpdateCapsuleConvexConstraint(const FCapsule& A, const FRigidTransform3& ATM, const T_ImplicitB& B, const FRigidTransform3& BTM, const FReal Dt, FPBDCollisionConstraint& Constraint)
			{
				// We should have called the specialized update function and not gone through the generic path
				ensure(false);
			}
		};

		// One-shot manifold convex-convex update
		template<>
		struct TCapsuleConvexUpdater<true>
		{
			template<typename ConvexType>
			static void UpdateCapsuleConvexConstraint(const FCapsule& A, const FRigidTransform3& ATransform, const ConvexType& B, const FRigidTransform3& BTransform, const FReal Dt, FPBDCollisionConstraint& Constraint)
			{
				if (Constraint.GetUseManifold())
				{
					FContactPointManifold ContactPoints;
					ConstructCapsuleConvexOneShotManifold<ConvexType>(A, ATransform, B, BTransform, Constraint.GetCullDistance(), ContactPoints);

					Constraint.ResetActiveManifoldContacts();
					for (const FContactPoint& ContactPoint : ContactPoints)
					{
						Constraint.AddOneshotManifoldContact(ContactPoint);
					}
				}
				else
				{
					UpdateContactPoint(Constraint, CapsuleConvexContactPoint(A, ATransform, B, BTransform), Dt);
				}
			}
		};

		// Another helper required by UpdateCapsuleGenericConvexConstraintHelper which uses CastHelper and does not have any typedefs 
		// for the concrete implicit types, so we need to rely on type deduction from the compiler.
		struct FCapsuleConvexUpdaterCaller
		{
			template<typename T_ImplicitA, typename T_ImplicitB>
			static void Update(const T_ImplicitA& A, const FRigidTransform3& ATM, const T_ImplicitB& B, const FRigidTransform3& BTM, const FReal Dt, FPBDCollisionConstraint& Constraint)
			{
				using FCapsuleConvexUpdater = TCapsuleConvexUpdater<TCapsuleImplicitPairTraits<T_ImplicitB>::bSupportsGenericOneShotManifoldPath>;
				FCapsuleConvexUpdater::UpdateCapsuleConvexConstraint(A, ATM, B, BTM, Dt, Constraint);
			}
		};

		// Unwrap the many convex types, including scaled, and call the appropriate update which depends on the concrete types
		void UpdateCapsuleGenericConvexConstraintHelper(const FCapsule& A, const FRigidTransform3& ATM, const FImplicitObject& B, const FRigidTransform3& BTM, const FReal Dt, FPBDCollisionConstraint& Constraint)
		{
			// This expands to a switch that calls the inner function with the appropriate concrete implicit type
			Utilities::CastHelperNoUnwrap(B, BTM, [&](const auto& BDowncast, const FRigidTransform3& BFullTM)
				{
					FCapsuleConvexUpdaterCaller::Update(A, ATM, BDowncast, BFullTM, Dt, Constraint);
				});
		}

		//
		// Capsule-HeightField
		//

		void UpdateCapsuleHeightFieldConstraint(const FCapsule& A, const FRigidTransform3& ATransform, const FHeightField& B, const FRigidTransform3& BTransform, const FReal Dt, FPBDCollisionConstraint& Constraint)
		{
			CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_Collisions_UpdateCapsuleHeightFieldConstraint, ConstraintsDetailedStats);
			if (Constraint.GetUseManifold())
			{
				ConstructQuadraticConvexHeightFieldOneShotManifold(A, ATransform, B, BTransform, Dt, Constraint);
			}
			else
			{
				UpdateContactPoint(Constraint, CapsuleHeightFieldContactPoint(A, ATransform, B, BTransform, Constraint.GetCullDistance()), Dt);
			}
		}


		void UpdateCapsuleHeightFieldConstraintSwept(TGeometryParticleHandle<FReal, 3>* Particle0, const FCapsule& A, const FRigidTransform3& ATransform, const FHeightField& B, const FRigidTransform3& BTransform, const FVec3& Dir, const FReal Length, const FReal Dt, FPBDCollisionConstraint& Constraint)
		{
			CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_Collisions_UpdateCapsuleHeightFieldConstraintSwept, ConstraintsDetailedStats);
			FReal TOI = FPBDCollisionConstraint::MaxTOI;
			UpdateContactPointNoCull(Constraint, GJKImplicitSweptContactPoint(A, ATransform, B, BTransform, Dir, Length, Constraint.GetCCDEnablePenetration(), Constraint.GetCCDTargetPenetration(), TOI), Dt, CCDNoCullAllShapePairs);
			Constraint.SetCCDTimeOfImpact(TOI);
		}

		//
		// Capsule-TriangleMesh
		//

		template <typename TriMeshType>
		void UpdateCapsuleTriangleMeshConstraint(const FCapsule& Capsule0, const FRigidTransform3& WorldTransform0, const TriMeshType& TriangleMesh1, const FRigidTransform3& WorldTransform1, const FReal Dt, FPBDCollisionConstraint& Constraint)
		{
			// @todo(chaos): restitution padding
			CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_Collisions_UpdateCapsuleTriangleMeshConstraint, ConstraintsDetailedStats);
			if (Constraint.GetUseManifold())
			{
				ConstructQuadraticConvexTriMeshOneShotManifold(Capsule0, WorldTransform0, TriangleMesh1, WorldTransform1, Dt, Constraint);
			}
			else
			{
				UpdateContactPoint(Constraint, CapsuleTriangleMeshContactPoint(Capsule0, WorldTransform0, TriangleMesh1, WorldTransform1, Constraint.GetCullDistance()), Dt);
			}
		}

		template <typename TriMeshType>
		void UpdateCapsuleTriangleMeshConstraintSwept(TGeometryParticleHandle<FReal, 3>* Particle0, const FCapsule& Capsule0, const FRigidTransform3& WorldTransform0, const TriMeshType& TriangleMesh1, const FRigidTransform3& WorldTransform1, const FVec3& Dir, const FReal Length, const FReal Dt, FPBDCollisionConstraint& Constraint)
		{
			CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_Collisions_UpdateCapsuleTriangleMeshConstraint, ConstraintsDetailedStats);
			FReal TOI = FPBDCollisionConstraint::MaxTOI;
			UpdateContactPointNoCull(Constraint, CapsuleTriangleMeshSweptContactPoint(Capsule0, WorldTransform0, TriangleMesh1, WorldTransform1, Dir, Length, Constraint.GetCCDEnablePenetration(), Constraint.GetCCDTargetPenetration(), TOI), Dt, CCDNoCullAllShapePairs);
			Constraint.SetCCDTimeOfImpact(TOI);
		}

		//
		// Generic Convex - Convex (actual concrete type could be anything)
		//

		void UpdateGenericConvexConvexConstraint(const FImplicitObject& Implicit0, const FRigidTransform3& WorldTransform0, const FImplicitObject& Implicit1, const FRigidTransform3& WorldTransform1, const FReal Dt, FPBDCollisionConstraint& Constraint)
		{
			CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_Collisions_UpdateGenericConvexConvexConstraint, ConstraintsDetailedStats);

			UpdateGenericConvexConvexConstraintHelper(Implicit0, WorldTransform0, Implicit1, WorldTransform1, Dt, Constraint);
		}

		void UpdateGenericConvexConvexConstraintSwept(TGeometryParticleHandle<FReal, 3>* Particle0, const FImplicitObject& Implicit0, const FRigidTransform3& StartWorldTransform0, TGeometryParticleHandle<FReal, 3>* Particle1, const FImplicitObject& Implicit1, const FRigidTransform3& StartWorldTransform1, const FVec3& Dir, const FReal Length, const FReal Dt, FPBDCollisionConstraint& Constraint)
		{
			CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_Collisions_UpdateGenericConvexConvexConstraintSwept, ConstraintsDetailedStats);
			FReal TOI = FPBDCollisionConstraint::MaxTOI;
			const FRigidTransform3& EndWorldTransform0 = Constraint.GetShapeWorldTransform0();
			const FRigidTransform3& EndWorldTransform1 = Constraint.GetShapeWorldTransform1();
			UpdateContactPointNoCull(Constraint, GenericConvexConvexContactPointSwept(Implicit0, StartWorldTransform0, EndWorldTransform0, Implicit1, StartWorldTransform1, EndWorldTransform1, Dir, Length, Constraint.GetCCDEnablePenetration(), Constraint.GetCCDTargetPenetration(), TOI), Dt);
			Constraint.SetCCDTimeOfImpact(TOI);
		}

		//
		// Convex - HeightField
		//

		void UpdateConvexHeightFieldConstraint(const FImplicitObject& A, const FRigidTransform3& ATransform, const FHeightField& B, const FRigidTransform3& BTransform, const FReal Dt, FPBDCollisionConstraint& Constraint)
		{
			CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_Collisions_UpdateConvexHeightFieldConstraint, ConstraintsDetailedStats);
			if (Constraint.GetUseManifold())
			{
				ConstructPlanarConvexHeightFieldOneShotManifold(A, ATransform, B, BTransform, Constraint);
			}
			else
			{
				UpdateContactPoint(Constraint, ConvexHeightFieldContactPoint(A, ATransform, B, BTransform, Constraint.GetCullDistance()), Dt);
			}
		}


		void UpdateConvexHeightFieldConstraintSwept(TGeometryParticleHandle<FReal, 3>* Particle0, const FImplicitObject& A, const FRigidTransform3& ATransform, const FHeightField& B, const FRigidTransform3& BTransform, const FVec3& Dir, const FReal Length, const FReal Dt, FPBDCollisionConstraint& Constraint)
		{
			CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_Collisions_UpdateConvexHeightFieldConstraintSwept, ConstraintsDetailedStats);
			FReal TOI = FPBDCollisionConstraint::MaxTOI;
			UpdateContactPointNoCull(Constraint, GJKImplicitSweptContactPoint(A, ATransform, B, BTransform, Dir, Length, Constraint.GetCCDEnablePenetration(), Constraint.GetCCDTargetPenetration(), TOI), Dt, CCDNoCullAllShapePairs);
			Constraint.SetCCDTimeOfImpact(TOI);
		}

		//
		// Convex-TriangleMesh
		//

		void UpdateConvexTriangleMeshConstraint(const FImplicitObject& Convex0, const FRigidTransform3& WorldTransform0, const FImplicitObject& TriangleMesh1, const FRigidTransform3& WorldTransform1, const FReal Dt, FPBDCollisionConstraint& Constraint)
		{
			CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_Collisions_UpdateConvexTriangleMeshConstraint, ConstraintsDetailedStats);
			if (Constraint.GetUseManifold())
			{
				ConstructPlanarConvexTriMeshOneShotManifold(Convex0, WorldTransform0, TriangleMesh1, WorldTransform1, Constraint);
			}
			else
			{
				UpdateContactPoint(Constraint, ConvexTriangleMeshContactPoint(Convex0, WorldTransform0, TriangleMesh1, WorldTransform1, Constraint.GetCullDistance()), Dt);
			}
		}

		// Sweeps convex against trimesh
		template <typename TriMeshType>
		void UpdateConvexTriangleMeshConstraintSwept(TGeometryParticleHandle<FReal, 3>* Particle0, const FImplicitObject& Convex0, const FRigidTransform3& WorldTransform0, const TriMeshType& TriangleMesh1, const FRigidTransform3& WorldTransform1, const FVec3& Dir, const FReal Length, const FReal Dt, FPBDCollisionConstraint& Constraint)
		{
			CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_Collisions_UpdateConvexTriangleMeshConstraintSwept, ConstraintsDetailedStats);
			FReal TOI = FPBDCollisionConstraint::MaxTOI;
			UpdateContactPointNoCull(Constraint, ConvexTriangleMeshSweptContactPoint(Convex0, WorldTransform0, TriangleMesh1, WorldTransform1, Dir, Length, Constraint.GetCCDEnablePenetration(), Constraint.GetCCDTargetPenetration(), TOI), Dt, CCDNoCullAllShapePairs);
			Constraint.SetCCDTimeOfImpact(TOI);
		}

		//
		// Levelset-Levelset
		//

		void UpdateLevelsetLevelsetConstraint(const FRigidTransform3& WorldTransform0, const FRigidTransform3& WorldTransform1, const FReal Dt, FPBDCollisionConstraint& Constraint)
		{
			SCOPE_CYCLE_COUNTER(STAT_UpdateLevelsetLevelsetConstraint);

			// @todo(chaos): get rid of this
			FRigidTransform3 ParticlesTM = WorldTransform0;
			if (!(ensure(!FMath::IsNaN(ParticlesTM.GetTranslation().X)) && ensure(!FMath::IsNaN(ParticlesTM.GetTranslation().Y)) && ensure(!FMath::IsNaN(ParticlesTM.GetTranslation().Z))))
			{
				return;
			}
			FRigidTransform3 LevelsetTM = WorldTransform1;
			if (!(ensure(!FMath::IsNaN(LevelsetTM.GetTranslation().X)) && ensure(!FMath::IsNaN(LevelsetTM.GetTranslation().Y)) && ensure(!FMath::IsNaN(LevelsetTM.GetTranslation().Z))))
			{
				return;
			}

			// NOTE: We are assuming that if only one body has collision particles, it is the first one. This implies
			// that the first body is dynamic (currently this is true)
			const FBVHParticles* SampleParticles = Constraint.GetCollisionParticles0();
			if (SampleParticles)
			{
				// Sample particles are in Particle(Actor) space, but SolverBody state in in CoM space
				// The transforms passed in are the Particle transforms but concatenatated with the shape transform(s) so we need
				// to regenerate the Particle transform without those shape transforms. We should be able to eliminate this...
				FGenericParticleHandle Particle0 = Constraint.GetParticle0();
				// @todo(chaos): we should not be accessing the particle here (or anywhere in collision detection)
				// The two paths here are required for unit testing. Units tests don't always set up the solver bodies, but this function is
				// usually called in the solver loop where solver bodies must be used.
				if (Constraint.GetSolverBody0() != nullptr)
				{
					const FRigidTransform3 CoMToParticleTransform = FParticleUtilities::ParticleLocalToCoMLocal(Particle0, FRigidTransform3::Identity);
					ParticlesTM = CoMToParticleTransform * FRigidTransform3(Constraint.GetSolverBody0()->CorrectedP(), Constraint.GetSolverBody0()->CorrectedQ());
				}
				else
				{
					ParticlesTM = FParticleUtilities::GetActorWorldTransform(Particle0);
				}
			}

			if (SampleParticles)
			{
				const FImplicitObject* Obj1 = Constraint.GetImplicit1();
				FContactPoint ContactPoint = SampleObject<ECollisionUpdateType::Deepest>(*Obj1, LevelsetTM, *SampleParticles, ParticlesTM, Constraint.GetCullDistance());
				UpdateContactPoint(Constraint, ContactPoint, Dt);
			}
		}

		//
		// Constraint API
		//

		// Run collision detection for the specified constraint to update the nearest contact point.
		// NOTE: Transforms are world space shape transforms
		// @todo(chaos): use a lookup table?
		inline void UpdateConstraintFromGeometryImpl(FPBDCollisionConstraint& Constraint, const FRigidTransform3& WorldTransform0, const FRigidTransform3& WorldTransform1, const FReal Dt)
		{
			if (!Constraint.GetImplicit0() || !Constraint.GetImplicit1())
			{
				return;
			}

			const FImplicitObject& Implicit0 = *Constraint.GetImplicit0();
			const FImplicitObject& Implicit1 = *Constraint.GetImplicit1();

			switch (Constraint.GetShapesType())
			{
			case EContactShapesType::SphereSphere:
				UpdateSphereSphereConstraint(*Implicit0.template GetObject<FSphere>(), WorldTransform0, *Implicit1.template GetObject<FSphere>(), WorldTransform1, Dt, Constraint);
				break;
			case EContactShapesType::SphereCapsule:
				UpdateSphereCapsuleConstraint(*Implicit0.template GetObject<FSphere>(), WorldTransform0, *Implicit1.template GetObject<FCapsule>(), WorldTransform1, Dt, Constraint);
				break;
			case EContactShapesType::SphereBox:
				UpdateSphereBoxConstraint(*Implicit0.template GetObject<FSphere>(), WorldTransform0, *Implicit1.template GetObject<TBox<FReal, 3>>(), WorldTransform1, Dt, Constraint);
				break;
			case EContactShapesType::SphereConvex:
				UpdateSphereConvexConstraint(*Implicit0.template GetObject<FSphere>(), WorldTransform0, Implicit1, WorldTransform1, Dt, Constraint);
				break;
			case EContactShapesType::SphereTriMesh:
				if (const TImplicitObjectScaled<FTriangleMeshImplicitObject>* ScaledTriMesh = Implicit1.template GetObject<const TImplicitObjectScaled<FTriangleMeshImplicitObject> >())
				{
					UpdateSphereTriangleMeshConstraint(*Implicit0.template GetObject<FSphere>(), WorldTransform0, *ScaledTriMesh, WorldTransform1, Dt, Constraint);
					break;
				}
				else if (const FTriangleMeshImplicitObject* TriangleMeshImplicit = Implicit1.template GetObject<const FTriangleMeshImplicitObject>())
				{
					UpdateSphereTriangleMeshConstraint(*Implicit0.template GetObject<FSphere>(), WorldTransform0, *TriangleMeshImplicit, WorldTransform1, Dt, Constraint);
					break;
				}
				ensure(false);
				break;
			case EContactShapesType::SphereHeightField:
				UpdateSphereHeightFieldConstraint(*Implicit0.template GetObject<FSphere>(), WorldTransform0, *Implicit1.template GetObject< FHeightField >(), WorldTransform1, Dt, Constraint);
				break;
			case EContactShapesType::SpherePlane:
				UpdateSpherePlaneConstraint(*Implicit0.template GetObject<FSphere>(), WorldTransform0, *Implicit1.template GetObject<TPlane<FReal, 3>>(), WorldTransform1, Dt, Constraint);
				break;
			case EContactShapesType::CapsuleCapsule:
				UpdateCapsuleCapsuleConstraint(*Implicit0.template GetObject<FCapsule>(), WorldTransform0, *Implicit1.template GetObject<FCapsule>(), WorldTransform1, Dt, Constraint);
				break;
			case EContactShapesType::CapsuleBox:
				UpdateCapsuleBoxConstraint(*Implicit0.template GetObject<FCapsule>(), WorldTransform0, *Implicit1.template GetObject<TBox<FReal, 3>>(), WorldTransform1, Dt, Constraint);
				break;
			case EContactShapesType::CapsuleConvex:
				UpdateCapsuleGenericConvexConstraintHelper(*Implicit0.template GetObject<FCapsule>(), WorldTransform0, Implicit1, WorldTransform1, Dt, Constraint);
				break;
			case EContactShapesType::CapsuleTriMesh:
				if (const TImplicitObjectScaled<FTriangleMeshImplicitObject>* ScaledTriMesh = Implicit1.template GetObject<const TImplicitObjectScaled<FTriangleMeshImplicitObject> >())
				{
					UpdateCapsuleTriangleMeshConstraint(*Implicit0.template GetObject<FCapsule>(), WorldTransform0, *ScaledTriMesh, WorldTransform1, Dt, Constraint);
					break;
				}
				else if (const FTriangleMeshImplicitObject* TriangleMeshImplicit = Implicit1.template GetObject<const FTriangleMeshImplicitObject>())
				{
					UpdateCapsuleTriangleMeshConstraint(*Implicit0.template GetObject<FCapsule>(), WorldTransform0, *TriangleMeshImplicit, WorldTransform1, Dt, Constraint);
					break;
				}
				ensure(false);
				break;
			case EContactShapesType::CapsuleHeightField:
				UpdateCapsuleHeightFieldConstraint(*Implicit0.template GetObject<FCapsule>(), WorldTransform0, *Implicit1.template GetObject< FHeightField >(), WorldTransform1, Dt, Constraint);
				break;
			case EContactShapesType::BoxBox:
				UpdateBoxBoxConstraint(*Implicit0.template GetObject<TBox<FReal, 3>>(), WorldTransform0, *Implicit1.template GetObject<TBox<FReal, 3>>(), WorldTransform1, Dt, Constraint);
				break;
			case EContactShapesType::BoxConvex:
				UpdateGenericConvexConvexConstraint(Implicit0, WorldTransform0, Implicit1, WorldTransform1, Dt, Constraint);
				break;
			case EContactShapesType::BoxTriMesh:
				if (const TImplicitObjectScaled<FTriangleMeshImplicitObject>* ScaledTriMesh = Implicit1.template GetObject<const TImplicitObjectScaled<FTriangleMeshImplicitObject> >())
				{
					UpdateBoxTriangleMeshConstraint(*Implicit0.template GetObject<TBox<FReal, 3>>(), WorldTransform0, *ScaledTriMesh, WorldTransform1, Dt, Constraint);
					break;
				}
				else if (const FTriangleMeshImplicitObject* TriangleMeshImplicit = Implicit1.template GetObject<const FTriangleMeshImplicitObject>())
				{
					UpdateBoxTriangleMeshConstraint(*Implicit0.template GetObject<TBox<FReal, 3>>(), WorldTransform0, *TriangleMeshImplicit, WorldTransform1, Dt, Constraint);
					break;
				}
				ensure(false);
				break;
			case EContactShapesType::BoxHeightField:
				UpdateBoxHeightFieldConstraint(*Implicit0.template GetObject<TBox<FReal, 3>>(), WorldTransform0, *Implicit1.template GetObject< FHeightField >(), WorldTransform1, Dt, Constraint);
				break;
			case EContactShapesType::BoxPlane:
				UpdateBoxPlaneConstraint(*Implicit0.template GetObject<TBox<FReal, 3>>(), WorldTransform0, *Implicit1.template GetObject<TPlane<FReal, 3>>(), WorldTransform1, Dt, Constraint);
				break;
			case EContactShapesType::GenericConvexConvex:
				UpdateGenericConvexConvexConstraint(Implicit0, WorldTransform0, Implicit1, WorldTransform1, Dt, Constraint);
				break;
			case EContactShapesType::ConvexTriMesh:
				UpdateConvexTriangleMeshConstraint(Implicit0, WorldTransform0, Implicit1, WorldTransform1, Dt, Constraint);
				break;
			case EContactShapesType::ConvexHeightField:
				UpdateConvexHeightFieldConstraint(Implicit0, WorldTransform0, *Implicit1.template GetObject< FHeightField >(), WorldTransform1, Dt, Constraint);
				break;
			case EContactShapesType::LevelSetLevelSet:
				UpdateLevelsetLevelsetConstraint(WorldTransform0, WorldTransform1, Dt, Constraint);
				break;
			default:
				// Switch needs updating....
				ensure(false);
				break;
			}
		}

		// Run collision detection for the swept constraints
		// NOTE: Transforms are world space shape transforms
		
		inline bool UpdateConstraintFromGeometrySweptImpl(FPBDCollisionConstraint& Constraint, const FRigidTransform3& WorldTransform0, const FRigidTransform3& WorldTransform1, const FReal Dt)
		{
			INC_DWORD_STAT(STAT_ChaosCollisionCounter_NumContactUpdates);

			TGeometryParticleHandle<FReal, 3>* Particle0 = Constraint.GetParticle0();
			TGeometryParticleHandle<FReal, 3>* Particle1 = Constraint.GetParticle1();
			const FImplicitObject& Implicit0 = *Constraint.GetImplicit0();
			const FImplicitObject& Implicit1 = *Constraint.GetImplicit1();

			const FVec3 DeltaX0 = Constraint.GetShapeWorldTransform0().GetTranslation() - WorldTransform0.GetLocation();
			const FVec3 DeltaX1 = Constraint.GetShapeWorldTransform1().GetTranslation() - WorldTransform1.GetLocation();
			FVec3 DirCCD = DeltaX0 - DeltaX1;
			const FReal LengthCCD = DirCCD.SafeNormalize();
			const bool bUseCCD = (LengthCCD > 0);

			if (!bChaosCCDResetTOIFix)
			{
				Constraint.ResetCCDTimeOfImpact();
			}

			if (bUseCCD)
			{
				const EContactShapesType ShapePairType = Constraint.GetShapesType();
				switch (ShapePairType)
				{
				case EContactShapesType::SphereSphere:
				case EContactShapesType::SphereCapsule:
				case EContactShapesType::SphereBox:
				case EContactShapesType::SphereConvex:
				case EContactShapesType::SpherePlane:
				case EContactShapesType::CapsuleCapsule:
				case EContactShapesType::CapsuleBox:
				case EContactShapesType::CapsuleConvex:
				case EContactShapesType::BoxBox:
				case EContactShapesType::BoxConvex:
				case EContactShapesType::BoxPlane:
				case EContactShapesType::GenericConvexConvex:
					UpdateGenericConvexConvexConstraintSwept(Particle0, Implicit0, WorldTransform0, Particle1, Implicit1, WorldTransform1, DirCCD, LengthCCD, Dt, Constraint);
					return true;
				case EContactShapesType::SphereHeightField:
				{
					const FSphere* Object0 = Implicit0.template GetObject<const FSphere>();
					const FHeightField* Object1 = Implicit1.template GetObject<const FHeightField>();
					UpdateSphereHeightFieldConstraintSwept(Particle0, *Object0, WorldTransform0, *Object1, WorldTransform1, DirCCD, LengthCCD, Dt, Constraint);
					return true;
				}
				case EContactShapesType::CapsuleHeightField:
				{
					const FCapsule* Object0 = Implicit0.template GetObject<const FCapsule >();
					const FHeightField* Object1 = Implicit1.template GetObject<const FHeightField >();
					UpdateCapsuleHeightFieldConstraintSwept(Particle0, *Object0, WorldTransform0, *Object1, WorldTransform1, DirCCD, LengthCCD, Dt, Constraint);
					return true;
				}
				case EContactShapesType::SphereTriMesh:
				{
					const FSphere* Object0 = Implicit0.template GetObject<const FSphere>();
					if (const TImplicitObjectScaled<FTriangleMeshImplicitObject>* ScaledTriangleMesh = Implicit1.template GetObject<const TImplicitObjectScaled<FTriangleMeshImplicitObject>>())
					{
						UpdateSphereTriangleMeshConstraintSwept(Particle0, *Object0, WorldTransform0, *ScaledTriangleMesh, WorldTransform1, DirCCD, LengthCCD, Dt, Constraint);
					}
					else if (const FTriangleMeshImplicitObject* TriangleMesh = Implicit1.template GetObject<const FTriangleMeshImplicitObject>())
					{
						UpdateSphereTriangleMeshConstraintSwept(Particle0, *Object0, WorldTransform0, *TriangleMesh, WorldTransform1, DirCCD, LengthCCD, Dt, Constraint);
					}
					else
					{
						ensure(false);
					}
					return true;
				}
				case EContactShapesType::CapsuleTriMesh:
				{
					const FCapsule* Object0 = Implicit0.template GetObject<const FCapsule >();
					if (const TImplicitObjectScaled<FTriangleMeshImplicitObject>* ScaledTriangleMesh = Implicit1.template GetObject<const TImplicitObjectScaled<FTriangleMeshImplicitObject>>())
					{
						UpdateCapsuleTriangleMeshConstraintSwept(Particle0, *Object0, WorldTransform0, *ScaledTriangleMesh, WorldTransform1, DirCCD, LengthCCD, Dt, Constraint);
					}
					else if (const FTriangleMeshImplicitObject* TriangleMesh = Implicit1.template GetObject<const FTriangleMeshImplicitObject>())
					{
						UpdateCapsuleTriangleMeshConstraintSwept(Particle0, *Object0, WorldTransform0, *TriangleMesh, WorldTransform1, DirCCD, LengthCCD, Dt, Constraint);
					}
					else
					{
						ensure(false);
					}
					return true;
				}
				case EContactShapesType::BoxHeightField:
				case EContactShapesType::ConvexHeightField:
				{
					const FHeightField* Object1 = Implicit1.template GetObject<const FHeightField >();
					UpdateConvexHeightFieldConstraintSwept(Particle0, Implicit0, WorldTransform0, *Object1, WorldTransform1, DirCCD, LengthCCD, Dt, Constraint);
					return true;
				}
				case EContactShapesType::BoxTriMesh:
				case EContactShapesType::ConvexTriMesh:
					if (const TImplicitObjectScaled<FTriangleMeshImplicitObject>* ScaledTriangleMesh = Implicit1.template GetObject<const TImplicitObjectScaled<FTriangleMeshImplicitObject>>())
					{
						UpdateConvexTriangleMeshConstraintSwept(Particle0, Implicit0, WorldTransform0, *ScaledTriangleMesh, WorldTransform1, DirCCD, LengthCCD, Dt, Constraint);
					}
					else if (const FTriangleMeshImplicitObject* TriangleMesh = Implicit1.template GetObject<const FTriangleMeshImplicitObject>())
					{
						UpdateConvexTriangleMeshConstraintSwept(Particle0, Implicit0, WorldTransform0, *TriangleMesh, WorldTransform1, DirCCD, LengthCCD, Dt, Constraint);
					}
					else
					{
						ensure(false);
					}
					return true;
				default:
					ensure(false);
					break;
				}
			}

			// CCD will not be used for this constraint this tick
			return false;
		}

		// Run collision detection for the specified constraint to update the nearest contact point.
		// NOTE: Transforms are world space shape transforms
		void UpdateConstraint(FPBDCollisionConstraint& Constraint, const FRigidTransform3& ShapeWorldTransform0, const FRigidTransform3& ShapeWorldTransform1, const FReal Dt)
		{
			UpdateConstraintFromGeometryImpl(Constraint, ShapeWorldTransform0, ShapeWorldTransform1, Dt);
		}

		bool UpdateConstraintSwept(FPBDCollisionConstraint& Constraint, const FRigidTransform3& ShapeStartWorldTransform0, const FRigidTransform3& ShapeStartWorldTransform1, const FReal Dt)
		{
			return UpdateConstraintFromGeometrySweptImpl(Constraint, ShapeStartWorldTransform0, ShapeStartWorldTransform1, Dt);
		}

		EContactShapesType CalculateShapePairType(const FGeometryParticleHandle* Particle0, const FImplicitObject* Implicit0, const FGeometryParticleHandle* Particle1, const FImplicitObject* Implicit1, bool &bOutSwap)
		{
			// What types do the implicits collide as?
			// NOTE: If the implicit is set to LevelSet but has no CollisionParticles we default to the implicits actual type
			const EImplicitObjectType Implicit0Type = Private::GetImplicitCollisionType(Particle0, Implicit0);
			const EImplicitObjectType Implicit1Type = Private::GetImplicitCollisionType(Particle1, Implicit1);

			//
			// Basic implicit pairs
			//

			bOutSwap = false;
			if (Implicit0Type == TBox<FReal, 3>::StaticType() && Implicit1Type == TBox<FReal, 3>::StaticType())
			{
				return EContactShapesType::BoxBox;
			}
			else if (Implicit0Type == TBox<FReal, 3>::StaticType() && Implicit1Type == FHeightField::StaticType())
			{
				return EContactShapesType::BoxHeightField;
			}
			else if (Implicit0Type == FHeightField::StaticType() && Implicit1Type == TBox<FReal, 3>::StaticType())
			{
				bOutSwap = true;
				return EContactShapesType::BoxHeightField;
			}
			else if (Implicit0Type == TBox<FReal, 3>::StaticType() && Implicit1Type == TPlane<FReal, 3>::StaticType())
			{
				return EContactShapesType::BoxPlane;
			}
			else if (Implicit0Type == TPlane<FReal, 3>::StaticType() && Implicit1Type == TBox<FReal, 3>::StaticType())
			{
				bOutSwap = true;
				return EContactShapesType::BoxPlane;
			}
			else if (Implicit0Type == FSphere::StaticType() && Implicit1Type == FSphere::StaticType())
			{
				return EContactShapesType::SphereSphere;
			}
			else if (Implicit0Type == FSphere::StaticType() && Implicit1Type == FHeightField::StaticType())
			{
				return EContactShapesType::SphereHeightField;
			}
			else if (Implicit0Type == FHeightField::StaticType() && Implicit1Type == FSphere::StaticType())
			{
				bOutSwap = true;
				return EContactShapesType::SphereHeightField;
			}
			else if (Implicit0Type == FSphere::StaticType() && Implicit1Type == TPlane<FReal, 3>::StaticType())
			{
				return EContactShapesType::SpherePlane;
			}
			else if (Implicit0Type == TPlane<FReal, 3>::StaticType() && Implicit1Type == FSphere::StaticType())
			{
				bOutSwap = true;
				return EContactShapesType::SpherePlane;
			}
			else if (Implicit0Type == FSphere::StaticType() && Implicit1Type == TBox<FReal, 3>::StaticType())
			{
				return EContactShapesType::SphereBox;
			}
			else if (Implicit0Type == TBox<FReal, 3>::StaticType() && Implicit1Type == FSphere::StaticType())
			{
				bOutSwap = true;
				return EContactShapesType::SphereBox;
			}
			else if (Implicit0Type == FSphere::StaticType() && Implicit1Type == FCapsule::StaticType())
			{
				return EContactShapesType::SphereCapsule;
			}
			else if (Implicit0Type == FCapsule::StaticType() && Implicit1Type == FSphere::StaticType())
			{
				bOutSwap = true;
				return EContactShapesType::SphereCapsule;
			}
			else if (Implicit0Type == FSphere::StaticType() && Implicit1Type == FImplicitConvex3::StaticType())
			{
				return EContactShapesType::SphereConvex;
			}
			else if (Implicit0Type == FImplicitConvex3::StaticType() && Implicit1Type == FSphere::StaticType())
			{
				bOutSwap = true;
				return EContactShapesType::SphereConvex;
			}
			else if (Implicit0Type == FCapsule::StaticType() && Implicit1Type == FCapsule::StaticType())
			{
				return EContactShapesType::CapsuleCapsule;
			}
			else if (Implicit0Type == FCapsule::StaticType() && Implicit1Type == TBox<FReal, 3>::StaticType())
			{
				return EContactShapesType::CapsuleBox;
			}
			else if (Implicit0Type == TBox<FReal, 3>::StaticType() && Implicit1Type == FCapsule::StaticType())
			{
				bOutSwap = true;
				return EContactShapesType::CapsuleBox;
			}
			else if (Implicit0Type == FCapsule::StaticType() && Implicit1Type == FImplicitConvex3::StaticType())
			{
				return EContactShapesType::CapsuleConvex;
			}
			else if (Implicit0Type == FImplicitConvex3::StaticType() && Implicit1Type == FCapsule::StaticType())
			{
				bOutSwap = true;
				return EContactShapesType::CapsuleConvex;
			}
			else if (Implicit0Type == FCapsule::StaticType() && Implicit1Type == FHeightField::StaticType())
			{
				return EContactShapesType::CapsuleHeightField;
			}
			else if (Implicit0Type == FHeightField::StaticType() && Implicit1Type == FCapsule::StaticType())
			{
				bOutSwap = true;
				return EContactShapesType::CapsuleHeightField;
			}
			else if (Implicit0Type == TBox<FReal, 3>::StaticType() && Implicit1Type == FTriangleMeshImplicitObject::StaticType())
			{
				return EContactShapesType::BoxTriMesh;
			}
			else if (Implicit0Type == FTriangleMeshImplicitObject::StaticType() && Implicit1Type == TBox<FReal, 3>::StaticType())
			{
				bOutSwap = true;
				return EContactShapesType::BoxTriMesh;
			}
			else if (Implicit0Type == FSphere::StaticType() && Implicit1Type == FTriangleMeshImplicitObject::StaticType())
			{
				return EContactShapesType::SphereTriMesh;
			}
			else if (Implicit0Type == FTriangleMeshImplicitObject::StaticType() && Implicit1Type == FSphere::StaticType())
			{
				bOutSwap = true;
				return EContactShapesType::SphereTriMesh;
			}
			else if (Implicit0Type == FCapsule::StaticType() && Implicit1Type == FTriangleMeshImplicitObject::StaticType())
			{
				return EContactShapesType::CapsuleTriMesh;
			}
			else if (Implicit0Type == FTriangleMeshImplicitObject::StaticType() && Implicit1Type == FCapsule::StaticType())
			{
				bOutSwap = true;
				return EContactShapesType::CapsuleTriMesh;
			}

			//
			// Convex implicits
			//

			const bool bIsConvex0 = (Implicit0 != nullptr) && Implicit0->IsConvex() && (Implicit0Type != ImplicitObjectType::LevelSet);
			const bool bIsConvex1 = (Implicit1 != nullptr) && Implicit1->IsConvex() && (Implicit1Type != ImplicitObjectType::LevelSet);
			if (bIsConvex0 && Implicit1Type == FHeightField::StaticType())
			{
				return EContactShapesType::ConvexHeightField;
			}
			else if (Implicit0Type == FHeightField::StaticType() && bIsConvex1)
			{
				bOutSwap = true;
				return EContactShapesType::ConvexHeightField;
			}
			else if (bIsConvex0 && Implicit1Type == FTriangleMeshImplicitObject::StaticType())
			{
				return EContactShapesType::ConvexTriMesh;
			}
			else if (Implicit0Type == FTriangleMeshImplicitObject::StaticType() && bIsConvex1)
			{
				bOutSwap = true;
				return EContactShapesType::ConvexTriMesh;
			}
			else if (bIsConvex0 && bIsConvex1)
			{
				return EContactShapesType::GenericConvexConvex;
			}

			//
			// Unions
			//
			
			if ((Implicit0Type == FImplicitObjectUnion::StaticType()) || (Implicit1Type == FImplicitObjectUnion::StaticType()))
			{
				// Unknown here is used to mean "run the recursive collision detection path" in the midphase
				// @todo(chaos): support FImplicitObjectUnion and FImplicitObjectUnionClustered more explicitly here
				return EContactShapesType::Unknown;
			}

			//
			// Levelsets
			//

			const bool bIsLevelSet0 = (Implicit0Type == ImplicitObjectType::LevelSet);
			const bool bIsLevelSet1 = (Implicit1Type == ImplicitObjectType::LevelSet);
			if (bIsLevelSet0 || bIsLevelSet1)
			{
				// If only one of the shapes is a levelset, it goes first
				bOutSwap = bIsLevelSet1 && !bIsLevelSet0;
				return EContactShapesType::LevelSetLevelSet;
			}

			// If we get here something is probably wrong.
			ensure(false);
			return EContactShapesType::Unknown;
		}


		//
		//
		// DEPRECATED STUFF
		//
		//

		EImplicitObjectType GetImplicitCollisionType(const FGeometryParticleHandle* Particle, const FImplicitObject* Implicit)
		{
			return Private::GetImplicitCollisionType(Particle, Implicit);
		}

		template<ECollisionUpdateType UpdateType>
		void UpdateConstraintFromGeometry(FPBDCollisionConstraint& Constraint, const FRigidTransform3& ParticleWorldTransform0, const FRigidTransform3& ParticleWorldTransform1, const FReal Dt)
		{
			const FRigidTransform3 WorldTransform0 = Constraint.GetShapeRelativeTransform0() * ParticleWorldTransform0;
			const FRigidTransform3 WorldTransform1 = Constraint.GetShapeRelativeTransform1() * ParticleWorldTransform1;
			Constraint.SetShapeWorldTransforms(WorldTransform0, WorldTransform1);

			UpdateConstraintFromGeometryImpl(Constraint, WorldTransform0, WorldTransform1, Dt);
		}
		template void UpdateConstraintFromGeometry<ECollisionUpdateType::Any>(FPBDCollisionConstraint& ConstraintBase, const FRigidTransform3& Transform0, const FRigidTransform3& Transform1, const FReal Dt);
		template void UpdateConstraintFromGeometry<ECollisionUpdateType::Deepest>(FPBDCollisionConstraint& ConstraintBase, const FRigidTransform3& Transform0, const FRigidTransform3& Transform1, const FReal Dt);

	} // Collisions


} // Chaos
