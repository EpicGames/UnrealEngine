// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/CollisionResolutionUtil.h"

#include "Chaos/ChaosPerfTest.h"
#include "Chaos/Defines.h"
#include "Chaos/ImplicitObjectUnion.h"
#include "Chaos/ImplicitObjectScaled.h"
#include "Chaos/SpatialAccelerationCollection.h"
#include "Chaos/Levelset.h"
#include "ChaosLog.h"
#include "ChaosStats.h"
#include "Stats/Stats.h" 

#if INTEL_ISPC
#include "PBDCollisionConstraints.ispc.generated.h"

static_assert(sizeof(ispc::FVector) == sizeof(Chaos::FVec3), "sizeof(ispc::FVector) != sizeof(Chaos::FVec3)");
static_assert(sizeof(ispc::FIntVector) == sizeof(Chaos::TVector<int32, 3>), "sizeof(ispc::FIntVector) != sizeof(Chaos::TVector<int32, 3>");
static_assert(sizeof(ispc::FTransform) == sizeof(Chaos::FRigidTransform3), "sizeof(ispc::FTransform) != sizeof(Chaos::FRigidTransform3)");
#endif

//PRAGMA_DISABLE_OPTIMIZATION

namespace Chaos
{
	namespace Collisions
	{

		FRigidTransform3 GetTransform(const TGeometryParticleHandle<FReal, 3>* Particle)
		{
			FGenericParticleHandle Generic = const_cast<FGeometryParticleHandle*>(Particle);
			return FRigidTransform3(Generic->P(), Generic->Q());
		}

		// Reference: Energy Stability and Fracture for Frame Rate Rigid Body Simulations (Su et al.) (3.2. Clamping Impulses)
		FVec3 GetEnergyClampedImpulse(const TPBDRigidParticleHandle<FReal, 3>* PBDRigid0, const TPBDRigidParticleHandle<FReal, 3>* PBDRigid1, const FVec3& Impulse, const FVec3& VectorToPoint1, const FVec3& VectorToPoint2, const FVec3& Velocity1, const FVec3& Velocity2)
		{
			const bool bIsRigidDynamic0 = PBDRigid0 && PBDRigid0->ObjectState() == EObjectStateType::Dynamic;
			const bool bIsRigidDynamic1 = PBDRigid1 && PBDRigid1->ObjectState() == EObjectStateType::Dynamic;

			FVec3 Jr0, Jr1, IInvJr0, IInvJr1;
			FReal ImpulseRatioNumerator0 = 0, ImpulseRatioNumerator1 = 0, ImpulseRatioDenom0 = 0, ImpulseRatioDenom1 = 0;
			FReal ImpulseSizeSQ = Impulse.SizeSquared();
			if (ImpulseSizeSQ < UE_SMALL_NUMBER)
			{
				return Impulse;
			}
			FVec3 KinematicVelocity = !bIsRigidDynamic0 ? Velocity1 : !bIsRigidDynamic1 ? Velocity2 : FVec3(0);
			if (bIsRigidDynamic0)
			{
				Jr0 = FVec3::CrossProduct(VectorToPoint1, Impulse);
				IInvJr0 = PBDRigid0->GetQ().RotateVector(FVec3(PBDRigid0->InvI()) * PBDRigid0->GetQ().UnrotateVector(Jr0));
				ImpulseRatioNumerator0 = FVec3::DotProduct(Impulse, PBDRigid0->GetV() - KinematicVelocity) + FVec3::DotProduct(Jr0, PBDRigid0->GetW());
				ImpulseRatioDenom0 = ImpulseSizeSQ / PBDRigid0->M() + FVec3::DotProduct(Jr0, IInvJr0);
			}
			if (bIsRigidDynamic1)
			{
				Jr1 = FVec3::CrossProduct(VectorToPoint2, Impulse);
				IInvJr1 = PBDRigid1->GetQ().RotateVector(FVec3(PBDRigid1->InvI()) * PBDRigid1->GetQ().UnrotateVector(Jr1));
				ImpulseRatioNumerator1 = FVec3::DotProduct(Impulse, PBDRigid1->GetV() - KinematicVelocity) + FVec3::DotProduct(Jr1, PBDRigid1->GetW());
				ImpulseRatioDenom1 = ImpulseSizeSQ / PBDRigid1->M() + FVec3::DotProduct(Jr1, IInvJr1);
			}
			FReal Numerator = -2 * (ImpulseRatioNumerator0 - ImpulseRatioNumerator1);
			if (Numerator <= 0)
			{
				return FVec3(0);
			}
			ensure(Numerator > 0);
			FReal Denominator = ImpulseRatioDenom0 + ImpulseRatioDenom1;
			return Numerator < Denominator ? (Impulse * Numerator / Denominator) : Impulse;
		}

		FVec3 GetEnergyClampedImpulse(
			const FVec3& Impulse, 
			FReal InvM0, 
			const FMatrix33& InvI0, 
			FReal InvM1, 
			const FMatrix33& InvI1,
			const FRotation3& Q0,
			const FVec3& V0,
			const FVec3& W0,
			const FRotation3& Q1,
			const FVec3& V1,
			const FVec3& W1,
			const FVec3& ContactOffset0,
			const FVec3& ContactOffset1, 
			const FVec3& ContactVelocity0, 
			const FVec3& ContactVelocity1)
		{
			FVec3 Jr0, Jr1, IInvJr0, IInvJr1;
			FReal ImpulseRatioNumerator0 = 0, ImpulseRatioNumerator1 = 0, ImpulseRatioDenom0 = 0, ImpulseRatioDenom1 = 0;
			FReal ImpulseSizeSQ = Impulse.SizeSquared();
			if (ImpulseSizeSQ < UE_SMALL_NUMBER)
			{
				return Impulse;
			}
			FVec3 KinematicVelocity = (InvM0 == 0.0f) ? ContactVelocity0 : (InvM1 == 0.0f) ? ContactVelocity1 : FVec3(0);
			if (InvM0 > 0.0f)
			{
				Jr0 = FVec3::CrossProduct(ContactOffset0, Impulse);
				IInvJr0 = Q0.RotateVector(InvI0 * Q0.UnrotateVector(Jr0));
				ImpulseRatioNumerator0 = FVec3::DotProduct(Impulse, V0 - KinematicVelocity) + FVec3::DotProduct(Jr0, W0);
				ImpulseRatioDenom0 = InvM0 * ImpulseSizeSQ + FVec3::DotProduct(Jr0, IInvJr0);
			}
			if (InvM1 > 0.0f)
			{
				Jr1 = FVec3::CrossProduct(ContactOffset1, Impulse);
				IInvJr1 = Q1.RotateVector(InvI1 * Q1.UnrotateVector(Jr1));
				ImpulseRatioNumerator1 = FVec3::DotProduct(Impulse, V1 - KinematicVelocity) + FVec3::DotProduct(Jr1, W1);
				ImpulseRatioDenom1 = InvM1 * ImpulseSizeSQ + FVec3::DotProduct(Jr1, IInvJr1);
			}
			FReal Numerator = -2.0f * (ImpulseRatioNumerator0 - ImpulseRatioNumerator1);
			if (Numerator <= 0)
			{
				return FVec3(0);
			}
			ensure(Numerator > 0);
			FReal Denominator = ImpulseRatioDenom0 + ImpulseRatioDenom1;
			return Numerator < Denominator ? (Impulse * Numerator / Denominator) : Impulse;
		}

		bool SampleObjectNoNormal(const FImplicitObject& Object, const FRigidTransform3& ObjectTransform, const FRigidTransform3& SampleToObjectTransform, const FVec3& SampleParticle, FReal Thickness, FContactPoint& Contact)
		{
			FVec3 LocalPoint = SampleToObjectTransform.TransformPositionNoScale(SampleParticle);
			FVec3 LocalNormal;
			FReal LocalPhi = Object.PhiWithNormal(LocalPoint, LocalNormal);

			if (LocalPhi < Contact.Phi)
			{
				Contact.Phi = LocalPhi;
				return true;
			}
			return false;
		}

		bool SampleObjectNormalAverageHelper(const FImplicitObject& Object, const FRigidTransform3& ObjectTransform, const FRigidTransform3& SampleToObjectTransform, const FVec3& SampleParticle, FReal Thickness, FReal& TotalThickness, FContactPoint& Contact)
		{
			FVec3 LocalPoint = SampleToObjectTransform.TransformPositionNoScale(SampleParticle);
			FVec3 LocalNormal;
			FReal LocalPhi = Object.PhiWithNormal(LocalPoint, LocalNormal);
			FReal LocalThickness = LocalPhi - Thickness;

			if (LocalThickness < -UE_KINDA_SMALL_NUMBER)
			{
				Contact.ShapeContactPoints[0] += LocalPoint * LocalThickness;
				TotalThickness += LocalThickness;
				return true;
			}
			return false;
		}

		DECLARE_CYCLE_STAT(TEXT("TPBDCollisionConstraints::UpdateLevelsetPartial"), STAT_UpdateLevelsetPartial, STATGROUP_ChaosWide);
		DECLARE_CYCLE_STAT(TEXT("TPBDCollisionConstraints::UpdateLevelsetFindParticles"), STAT_UpdateLevelsetFindParticles, STATGROUP_ChaosWide);
		DECLARE_CYCLE_STAT(TEXT("TPBDCollisionConstraints::UpdateLevelsetBVHTraversal"), STAT_UpdateLevelsetBVHTraversal, STATGROUP_ChaosWide);
		DECLARE_CYCLE_STAT(TEXT("TPBDCollisionConstraints::UpdateLevelsetSignedDistance"), STAT_UpdateLevelsetSignedDistance, STATGROUP_ChaosWide);
		DECLARE_CYCLE_STAT(TEXT("TPBDCollisionConstraints::UpdateLevelsetAll"), STAT_UpdateLevelsetAll, STATGROUP_ChaosWide);
		DECLARE_CYCLE_STAT(TEXT("TPBDCollisionConstraints::SampleObject"), STAT_SampleObject, STATGROUP_ChaosWide);

		int32 NormalAveraging = 0;
		FAutoConsoleVariableRef CVarNormalAveraging(TEXT("p.NormalAveraging2"), NormalAveraging, TEXT(""));

		int32 SampleMinParticlesForAcceleration = 2048;
		FAutoConsoleVariableRef CVarSampleMinParticlesForAcceleration(TEXT("p.SampleMinParticlesForAcceleration"), SampleMinParticlesForAcceleration, TEXT("The minimum number of particles needed before using an acceleration structure when sampling"));


#if INTEL_ISPC && 0
		// LWC_TODO: get ISPC levelset collision working again

		template<ECollisionUpdateType UpdateType>
		FContactPoint SampleObject(const FImplicitObject& Object, const FRigidTransform3& ObjectTransform, const FBVHParticles& SampleParticles, const FRigidTransform3& SampleParticlesTransform, FReal CullingDistance)
		{
			check(bRealTypeCompatibleWithISPC);
			SCOPE_CYCLE_COUNTER(STAT_SampleObject);

			FContactPoint Contact;
			FContactPoint AvgContact;

			Contact.ShapeContactPoints[0] = FVec3(0);
			Contact.ShapeContactPoints[1] = FVec3(0);
			Contact.ShapeContactNormal = FVec3(0);
			AvgContact.ShapeContactPoints[0] = FVec3(0);
			AvgContact.ShapeContactPoints[1] = FVec3(0);
			AvgContact.ShapeContactNormal = FVec3(0);
			AvgContact.Phi = CullingDistance;
			FReal WeightSum = FReal(0); // Sum of weights used for averaging (negative)

			int32 DeepestParticle = -1;

			const FRigidTransform3& SampleToObjectTM = SampleParticlesTransform.GetRelativeTransform(ObjectTransform);
			int32 NumParticles = SampleParticles.Size();

			if (NumParticles > SampleMinParticlesForAcceleration && Object.HasBoundingBox())
			{
				SCOPE_CYCLE_COUNTER(STAT_UpdateLevelsetPartial);
				FAABB3 ImplicitBox = Object.BoundingBox().TransformedAABB(ObjectTransform.GetRelativeTransform(SampleParticlesTransform));
				ImplicitBox.Thicken(CullingDistance);
				TArray<int32> PotentialParticles;
				{
					SCOPE_CYCLE_COUNTER(STAT_UpdateLevelsetFindParticles);
					PotentialParticles = SampleParticles.FindAllIntersections(ImplicitBox);
				}
				{
					SCOPE_CYCLE_COUNTER(STAT_UpdateLevelsetSignedDistance);

					if (Object.GetType() == ImplicitObjectType::LevelSet && PotentialParticles.Num() > 0)
					{
						//QUICK_SCOPE_CYCLE_COUNTER(STAT_LevelSet);
						const FLevelSet* LevelSet = Object.GetObject<FLevelSet>();
						const TUniformGrid<FReal, 3>& Grid = LevelSet->GetGrid();

						if (NormalAveraging && UpdateType != ECollisionUpdateType::Any)
						{
							ispc::SampleLevelSetNormalAverage(
								(ispc::FVector&)Grid.MinCorner(),
								(ispc::FVector&)Grid.MaxCorner(),
								(ispc::FVector&)Grid.Dx(),
								(ispc::FIntVector&)Grid.Counts(),
								(ispc::TArrayND*)&LevelSet->GetPhiArray(),
								(ispc::FTransform&)SampleToObjectTM,
								(ispc::FVector*)&SampleParticles.XArray()[0],
								&PotentialParticles[0],
								CullingDistance,
								WeightSum,
								(ispc::FVector&)AvgContact.ShapeContactPoints[0],
								PotentialParticles.Num());
						}
						else
						{
							ispc::SampleLevelSetNoNormal(
								(ispc::FVector&)Grid.MinCorner(),
								(ispc::FVector&)Grid.MaxCorner(),
								(ispc::FVector&)Grid.Dx(),
								(ispc::FIntVector&)Grid.Counts(),
								(ispc::TArrayND*)&LevelSet->GetPhiArray(),
								(ispc::FTransform&)SampleToObjectTM,
								(ispc::FVector*)&SampleParticles.XArray()[0],
								&PotentialParticles[0],
								DeepestParticle,
								AvgContact.Phi,
								PotentialParticles.Num());

							if (UpdateType == ECollisionUpdateType::Any)
							{
								Contact.Phi = AvgContact.Phi;
								return Contact;
							}
						}
					}
					else if (Object.GetType() == ImplicitObjectType::Box && PotentialParticles.Num() > 0)
					{
						//QUICK_SCOPE_CYCLE_COUNTER(STAT_Box);
						const TBox<FReal, 3>* Box = Object.GetObject<Chaos::TBox<FReal, 3>>();
						const FVec3 BoxMin = Box->Min();
						const FVec3 BoxMax = Box->Max();

						if (NormalAveraging && UpdateType != ECollisionUpdateType::Any)
						{
							ispc::SampleBoxNormalAverage(
								(ispc::FVector&)BoxMin,
								(ispc::FVector&)BoxMax,
								(ispc::FTransform&)SampleToObjectTM,
								(ispc::FVector*)&SampleParticles.XArray()[0],
								&PotentialParticles[0],
								CullingDistance,
								WeightSum,
								(ispc::FVector&)AvgContact.ShapeContactPoints[0],
								PotentialParticles.Num());
						}
						else
						{
							ispc::SampleBoxNoNormal(
								(ispc::FVector&)BoxMin,
								(ispc::FVector&)BoxMax,
								(ispc::FTransform&)SampleToObjectTM,
								(ispc::FVector*)&SampleParticles.XArray()[0],
								&PotentialParticles[0],
								DeepestParticle,
								AvgContact.Phi,
								PotentialParticles.Num());

							if (UpdateType == ECollisionUpdateType::Any)
							{
								Contact.Phi = AvgContact.Phi;
								return Contact;
							}
						}
					}
					else
					{
						//QUICK_SCOPE_CYCLE_COUNTER(STAT_Other);
						for (int32 i : PotentialParticles)
						{
							if (NormalAveraging && UpdateType != ECollisionUpdateType::Any)
							{
								SampleObjectNormalAverageHelper(Object, ObjectTransform, SampleToObjectTM, SampleParticles.X(i), CullingDistance, WeightSum, AvgContact);
							}
							else
							{
								if (SampleObjectNoNormal(Object, ObjectTransform, SampleToObjectTM, SampleParticles.X(i), CullingDistance, AvgContact))
								{
									DeepestParticle = i;
									if (UpdateType == ECollisionUpdateType::Any)
									{
										Contact.Phi = AvgContact.Phi;
										return Contact;
									}
								}
							}
						}
					}
				}
			}
			else
			{
				SCOPE_CYCLE_COUNTER(STAT_UpdateLevelsetAll);
				if (Object.GetType() == ImplicitObjectType::LevelSet && NumParticles > 0)
				{
					const FLevelSet* LevelSet = Object.GetObject<Chaos::FLevelSet>();
					const TUniformGrid<FReal, 3>& Grid = LevelSet->GetGrid();

					if (NormalAveraging && UpdateType != ECollisionUpdateType::Any)
					{
						ispc::SampleLevelSetNormalAverageAll(
							(ispc::FVector&)Grid.MinCorner(),
							(ispc::FVector&)Grid.MaxCorner(),
							(ispc::FVector&)Grid.Dx(),
							(ispc::FIntVector&)Grid.Counts(),
							(ispc::TArrayND*)&LevelSet->GetPhiArray(),
							(ispc::FTransform&)SampleToObjectTM,
							(ispc::FVector*)&SampleParticles.XArray()[0],
							CullingDistance,
							WeightSum,
							(ispc::FVector&)AvgContact.ShapeContactPoints[0],
							NumParticles);
					}
					else
					{
						ispc::SampleLevelSetNoNormalAll(
							(ispc::FVector&)Grid.MinCorner(),
							(ispc::FVector&)Grid.MaxCorner(),
							(ispc::FVector&)Grid.Dx(),
							(ispc::FIntVector&)Grid.Counts(),
							(ispc::TArrayND*)&LevelSet->GetPhiArray(),
							(ispc::FTransform&)SampleToObjectTM,
							(ispc::FVector*)&SampleParticles.XArray()[0],
							DeepestParticle,
							AvgContact.Phi,
							NumParticles);

						if (UpdateType == ECollisionUpdateType::Any)
						{
							Contact.Phi = AvgContact.Phi;
							return Contact;
						}
					}
				}
				else if (Object.GetType() == ImplicitObjectType::Plane && NumParticles > 0)
				{
					const TPlane<FReal, 3>* Plane = Object.GetObject<Chaos::TPlane<FReal, 3>>();

					if (NormalAveraging && UpdateType != ECollisionUpdateType::Any)
					{
						ispc::SamplePlaneNormalAverageAll(
							(ispc::FVector&)Plane->Normal(),
							(ispc::FVector&)Plane->X(),
							(ispc::FTransform&)SampleToObjectTM,
							(ispc::FVector*)&SampleParticles.XArray()[0],
							CullingDistance,
							WeightSum,
							(ispc::FVector&)AvgContact.ShapeContactPoints[0],
							NumParticles);
					}
					else
					{
						ispc::SamplePlaneNoNormalAll(
							(ispc::FVector&)Plane->Normal(),
							(ispc::FVector&)Plane->X(),
							(ispc::FTransform&)SampleToObjectTM,
							(ispc::FVector*)&SampleParticles.XArray()[0],
							DeepestParticle,
							AvgContact.Phi,
							NumParticles);

						if (UpdateType == ECollisionUpdateType::Any)
						{
							Contact.Phi = AvgContact.Phi;
							return Contact;
						}
					}
				}
				else if (Object.GetType() == ImplicitObjectType::Sphere && NumParticles > 0)
				{
					const FSphere* Sphere = Object.GetObject<Chaos::FSphere>();

					if (NormalAveraging && UpdateType != ECollisionUpdateType::Any)
					{
						ispc::SampleSphereNormalAverageAll(
							Sphere->GetRadius(),
							(ispc::FVector&)Sphere->GetCenter(),
							(ispc::FTransform&)SampleToObjectTM,
							(ispc::FVector*) & SampleParticles.XArray()[0],
							CullingDistance,
							WeightSum,
							(ispc::FVector&)AvgContact.ShapeContactPoints[0],
							NumParticles);
					}
					else
					{
						ispc::SampleSphereNoNormalAll(
							Sphere->GetRadius(),
							(ispc::FVector&)Sphere->GetCenter(),
							(ispc::FTransform&)SampleToObjectTM,
							(ispc::FVector*) & SampleParticles.XArray()[0],
							DeepestParticle,
							AvgContact.Phi,
							NumParticles);

						if (UpdateType == ECollisionUpdateType::Any)
						{
							Contact.Phi = AvgContact.Phi;
							return Contact;
						}
					}
				}
				else if (Object.GetType() == ImplicitObjectType::Box && NumParticles > 0)
				{
					const TBox<FReal, 3>* Box = Object.GetObject<Chaos::TBox<FReal, 3>>();
					const FVec3 BoxMin = Box->Min();
					const FVec3 BoxMax = Box->Max();

					if (NormalAveraging && UpdateType != ECollisionUpdateType::Any)
					{
						ispc::SampleBoxNormalAverageAll(
							(ispc::FVector&)BoxMin,
							(ispc::FVector&)BoxMax,
							(ispc::FTransform&)SampleToObjectTM,
							(ispc::FVector*)&SampleParticles.XArray()[0],
							CullingDistance,
							WeightSum,
							(ispc::FVector&)AvgContact.ShapeContactPoints[0],
							NumParticles);
					}
					else
					{
						ispc::SampleBoxNoNormalAll(
							(ispc::FVector&)BoxMin,
							(ispc::FVector&)BoxMax,
							(ispc::FTransform&)SampleToObjectTM,
							(ispc::FVector*)&SampleParticles.XArray()[0],
							DeepestParticle,
							AvgContact.Phi,
							NumParticles);

						if (UpdateType == ECollisionUpdateType::Any)
						{
							Contact.Phi = AvgContact.Phi;
							return Contact;
						}
					}
				}
				else
				{
					//QUICK_SCOPE_CYCLE_COUNTER(STAT_Other);
					for (int32 i = 0; i < NumParticles; ++i)
					{
						if (NormalAveraging && UpdateType != ECollisionUpdateType::Any)
						{
							SampleObjectNormalAverageHelper(Object, ObjectTransform, SampleToObjectTM, SampleParticles.X(i), CullingDistance, WeightSum, AvgContact);
						}
						else
						{
							if (SampleObjectNoNormal(Object, ObjectTransform, SampleToObjectTM, SampleParticles.X(i), CullingDistance, AvgContact))
							{
								DeepestParticle = i;
								if (UpdateType == ECollisionUpdateType::Any)
								{
									Contact.Phi = AvgContact.Phi;
									return Contact;
								}
							}
						}
					}
				}
			}

			if (NormalAveraging)
			{
				if (WeightSum < -KINDA_SMALL_NUMBER)
				{
					FVec3 LocalPoint = AvgContact.ShapeContactPoints[0] / WeightSum;
					FVec3 LocalNormal;
					const FReal NewPhi = Object.PhiWithNormal(LocalPoint, LocalNormal);
					if (NewPhi < Contact.Phi)
					{
						Contact.Phi = NewPhi;
						Contact.ShapeContactPoints[0] = SampleToObjectTM.InverseTransformPositionNoScale(LocalPoint);
						Contact.ShapeContactPoints[1] = LocalPoint - NewPhi * LocalNormal;
						Contact.ShapeContactNormal = LocalNormal;
					}
				}
				else
				{
					check(AvgContact.Phi >= CullingDistance);
				}
			}
			else if (AvgContact.Phi < CullingDistance)
			{
				check(DeepestParticle >= 0);
				FVec3 LocalPoint = SampleToObjectTM.TransformPositionNoScale(SampleParticles.X(DeepestParticle));
				FVec3 LocalNormal;
				Contact.Phi = Object.PhiWithNormal(LocalPoint, LocalNormal);
				Contact.ShapeContactPoints[0] = SampleParticles.X(DeepestParticle);
				Contact.ShapeContactPoints[1] = LocalPoint - Contact.Phi * LocalNormal;
				Contact.ShapeContactNormal = LocalNormal;
			}

			return Contact;
		}
#else		
		template <ECollisionUpdateType UpdateType>
		FContactPoint SampleObject(const FImplicitObject& Object, const FRigidTransform3& ObjectTransform, const FBVHParticles& SampleParticles, const FRigidTransform3& SampleParticlesTransform, FReal CullingDistance)
		{
			SCOPE_CYCLE_COUNTER(STAT_SampleObject);

			FContactPoint Contact;
			FContactPoint AvgContact;

			Contact.ShapeContactPoints[0] = FVec3(0);
			Contact.ShapeContactPoints[1] = FVec3(0);
			Contact.ShapeContactNormal = FVec3(0);
			AvgContact.ShapeContactPoints[0] = FVec3(0);
			AvgContact.ShapeContactPoints[1] = FVec3(0);
			AvgContact.ShapeContactNormal = FVec3(0);
			AvgContact.Phi = CullingDistance;
			FReal WeightSum = FReal(0); // Sum of weights used for averaging (negative)

			int32 DeepestParticle = -1;
			const int32 NumParticles = SampleParticles.Size();

			const FRigidTransform3 & SampleToObjectTM = SampleParticlesTransform.GetRelativeTransform(ObjectTransform);
			if (NumParticles > SampleMinParticlesForAcceleration && Object.HasBoundingBox())
			{
				SCOPE_CYCLE_COUNTER(STAT_UpdateLevelsetPartial);
				FAABB3 ImplicitBox = Object.BoundingBox().TransformedAABB(ObjectTransform.GetRelativeTransform(SampleParticlesTransform));
				ImplicitBox.Thicken(CullingDistance);
				TArray<int32> PotentialParticles;
				{
					SCOPE_CYCLE_COUNTER(STAT_UpdateLevelsetFindParticles);
					PotentialParticles = SampleParticles.FindAllIntersections(ImplicitBox);
				}
				{
					SCOPE_CYCLE_COUNTER(STAT_UpdateLevelsetSignedDistance);
					for (int32 i : PotentialParticles)
					{
						if (NormalAveraging && UpdateType != ECollisionUpdateType::Any)	//if we just want one don't bother with normal
						{
							SampleObjectNormalAverageHelper(Object, ObjectTransform, SampleToObjectTM, SampleParticles.GetX(i), CullingDistance, WeightSum, AvgContact);
						}
						else
						{
							if (SampleObjectNoNormal(Object, ObjectTransform, SampleToObjectTM, SampleParticles.GetX(i), CullingDistance, AvgContact))
							{
								DeepestParticle = i;
								if (UpdateType == ECollisionUpdateType::Any)
								{
									Contact.Phi = AvgContact.Phi;
									return Contact;
								}
							}
						}
					}
				}
			}
			else
			{
				SCOPE_CYCLE_COUNTER(STAT_UpdateLevelsetAll);
				for (int32 i = 0; i < NumParticles; ++i)
				{
					if (NormalAveraging && UpdateType != ECollisionUpdateType::Any)	//if we just want one don't bother with normal
					{
						const bool bInside = SampleObjectNormalAverageHelper(Object, ObjectTransform, SampleToObjectTM, SampleParticles.GetX(i), CullingDistance, WeightSum, AvgContact);
					}
					else
					{
						if (SampleObjectNoNormal(Object, ObjectTransform, SampleToObjectTM, SampleParticles.GetX(i), CullingDistance, AvgContact))
						{
							DeepestParticle = i;
							if (UpdateType == ECollisionUpdateType::Any)
							{
								Contact.Phi = AvgContact.Phi;
								return Contact;
							}
						}
					}
				}
			}

			if (NormalAveraging)
			{
				if (WeightSum < -UE_KINDA_SMALL_NUMBER)
				{
					FVec3 LocalPoint = AvgContact.ShapeContactPoints[0] / WeightSum;
					FVec3 LocalNormal;
					const FReal NewPhi = Object.PhiWithNormal(LocalPoint, LocalNormal);
					if (NewPhi < Contact.Phi)
					{
						Contact.Phi = NewPhi;
						Contact.ShapeContactPoints[0] = SampleToObjectTM.InverseTransformPositionNoScale(LocalPoint);
						Contact.ShapeContactPoints[1] = LocalPoint - NewPhi * LocalNormal; 
						Contact.ShapeContactNormal = LocalNormal;
					}
				}
				else
				{
					check(AvgContact.Phi >= CullingDistance);
				}
			}
			else if (AvgContact.Phi < CullingDistance)
			{
				check(DeepestParticle >= 0);
				FVec3 LocalPoint = SampleToObjectTM.TransformPositionNoScale(SampleParticles.GetX(DeepestParticle));
				FVec3 LocalNormal;
				Contact.Phi = Object.PhiWithNormal(LocalPoint, LocalNormal);
				Contact.ShapeContactPoints[0] = SampleParticles.GetX(DeepestParticle);
				Contact.ShapeContactPoints[1] = LocalPoint - Contact.Phi * LocalNormal;
				Contact.ShapeContactNormal = LocalNormal;
			}

			return Contact;
		}
#endif


		DECLARE_CYCLE_STAT(TEXT("TPBDCollisionConstraints::FindRelevantShapes"), STAT_FindRelevantShapes, STATGROUP_ChaosWide);
		TArray<Pair<const FImplicitObject*, FRigidTransform3>> FindRelevantShapes(const FImplicitObject* ParticleObj, const FRigidTransform3& ParticlesTM, const FImplicitObject& LevelsetObj, const FRigidTransform3& LevelsetTM, const FReal Thickness)
		{
			SCOPE_CYCLE_COUNTER(STAT_FindRelevantShapes);
			TArray<Pair<const FImplicitObject*, FRigidTransform3>> RelevantShapes;
			//find all levelset inner objects
			if (ParticleObj)
			{
				if (ParticleObj->HasBoundingBox())
				{
					const FRigidTransform3 ParticlesToLevelsetTM = ParticlesTM.GetRelativeTransform(LevelsetTM);
					FAABB3 ParticleBoundsInLevelset = ParticleObj->BoundingBox().TransformedAABB(ParticlesToLevelsetTM);
					ParticleBoundsInLevelset.Thicken(Thickness);
					{
						LevelsetObj.FindAllIntersectingObjects(RelevantShapes, ParticleBoundsInLevelset);
					}
				}
				else
				{
					LevelsetObj.AccumulateAllImplicitObjects(RelevantShapes, FRigidTransform3::Identity);
				}
			}
			else
			{
				//todo:compute bounds
				LevelsetObj.AccumulateAllImplicitObjects(RelevantShapes, FRigidTransform3::Identity);
			}

			return RelevantShapes;
		}

		template FContactPoint SampleObject<ECollisionUpdateType::Any>(const FImplicitObject& Object, const FRigidTransform3& ObjectTransform, const FBVHParticles& SampleParticles, const FRigidTransform3& SampleParticlesTransform, FReal CullingDistance);
		template FContactPoint SampleObject<ECollisionUpdateType::Deepest>(const FImplicitObject& Object, const FRigidTransform3& ObjectTransform, const FBVHParticles& SampleParticles, const FRigidTransform3& SampleParticlesTransform, FReal CullingDistance);

	} // Collisions

} // Chaos
