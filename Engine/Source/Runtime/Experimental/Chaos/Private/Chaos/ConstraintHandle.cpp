// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/ConstraintHandle.h"

#if CHAOS_CONSTRAINTHANDLE_DEBUG_ENABLED
// For debugging only
#include "Chaos/Collision/PBDCollisionConstraintHandle.h"
#endif

namespace Chaos
{
#if CHAOS_CONSTRAINTHANDLE_DEBUG_ENABLED
	void FConstraintHandleHolder::InitDebugData()
	{
		ConstraintType = nullptr;
		Particles[0] = nullptr;
		Particles[1] = nullptr;
		if (Handle != nullptr)
		{
			ConstraintType = &Handle->GetType();

			Particles[0] = Handle->GetConstrainedParticles()[0];
			Particles[1] = Handle->GetConstrainedParticles()[1];
		}
	}
#endif
}