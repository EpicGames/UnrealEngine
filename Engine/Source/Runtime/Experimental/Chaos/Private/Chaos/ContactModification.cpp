// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/ContactModification.h"
#include "Chaos/Collision/PBDCollisionConstraint.h"
#include "Chaos/Particle/ParticleUtilities.h"
#include "Chaos/PBDCollisionConstraints.h"
#include "Chaos/CollisionResolution.h"

namespace Chaos
{
	void FContactPairModifier::Disable()
	{
		Modifier->DisableConstraint(*Constraint);
	}

	void FContactPairModifier::Enable()
	{
		Modifier->EnableConstraint(*Constraint);
	}

	void FContactPairModifier::ConvertToProbe()
	{
		Modifier->ConvertToProbeConstraint(*Constraint);
	}

	void FContactPairModifier::ConvertToNonProbe()
	{
		Modifier->ConvertToNonProbeConstraint(*Constraint);
	}

	bool FContactPairModifier::GetIsProbe() const
	{
		return Constraint->GetIsProbe();
	}

	int32 FContactPairModifier::GetNumContacts() const
	{
		return Constraint->GetManifoldPoints().Num();
	}

	int32 FContactPairModifier::GetDeepestContactIndex() const
	{
		TArrayView<const FManifoldPoint> ManifoldPoints = Constraint->GetManifoldPoints();

		// We could use GetSeparation() here, but Phi avoids some computation.

		FReal DeepestSeparation = ManifoldPoints[0].ContactPoint.Phi;
		int32 DeepestIdx = 0;

		for (int32 Idx = 1; Idx < ManifoldPoints.Num(); ++Idx)
		{
			const FReal Separation = ManifoldPoints[Idx].ContactPoint.Phi;
			if (Separation < DeepestSeparation)
			{
				DeepestSeparation = Separation;
				DeepestIdx = Idx;
			}
		}

		return DeepestIdx;
	}

	bool FContactPairModifier::GetIsInitialContact() const
	{
		return Constraint->IsInitialContact();
	}

	void FContactPairModifier::SetIsInitialContact(bool bIsInitialContact) const
	{
		Constraint->SetModifierApplied();

		Constraint->SetIsInitialContact(bIsInitialContact);
	}

	const FImplicitObject* FContactPairModifier::GetContactGeometry(int32 ParticleIdx)
	{
		return Constraint->GetImplicit(ParticleIdx);
	}

	FRigidTransform3 FContactPairModifier::GetShapeToWorld(int32 ParticleIdx) const
	{
		TVec2<FGeometryParticleHandle*> Particles = GetParticlePair();
		FGeometryParticleHandle*& Particle = Particles[ParticleIdx];
		
		if (FPBDRigidParticleHandle* Rigid = Particle->CastToRigidParticle())
		{
			// Use PQ for rigids.
			return Constraint->GetShapeRelativeTransform(ParticleIdx) * Rigid->GetTransformPQ();
		}

		return Constraint->GetShapeRelativeTransform(ParticleIdx) * Particle->GetTransformXR();
	}

	FReal FContactPairModifier::GetSeparation(int32 ContactPointIdx) const
	{
		// Compute separation with distance between contact points.

		FVec3 WorldPos0, WorldPos1;
		GetWorldContactLocations(ContactPointIdx, WorldPos0, WorldPos1);
		const FVec3 Normal = GetWorldNormal(ContactPointIdx);
		return FVec3::DotProduct(Normal, (WorldPos0 - WorldPos1));
	}

	FReal FContactPairModifier::GetTargetSeparation(int32 ContactPointIdx) const
	{
		FManifoldPoint& ManifoldPoint = Constraint->GetManifoldPoint(ContactPointIdx);
		return ManifoldPoint.TargetPhi;
	}

	void FContactPairModifier::ModifyTargetSeparation(FReal TargetSeparation, int32 ContactPointIdx)
	{	
		Constraint->SetModifierApplied();

		FManifoldPoint& ManifoldPoint = Constraint->GetManifoldPoint(ContactPointIdx);
		ManifoldPoint.TargetPhi = FRealSingle(TargetSeparation);
	}

	FVec3 FContactPairModifier::GetWorldNormal(int32 ContactPointIdx) const
	{
		const FRigidTransform3& ShapeTransform1 = Constraint->GetShapeWorldTransform1();

		TArrayView<FManifoldPoint> ManifoldPoints = Constraint->GetManifoldPoints();
		FManifoldPoint& ManifoldPoint = ManifoldPoints[ContactPointIdx];

		return ShapeTransform1.TransformVectorNoScale(FVec3(ManifoldPoint.ContactPoint.ShapeContactNormal));
	}

	void FContactPairModifier::ModifyWorldNormal(const FVec3& Normal, int32 ContactPointIdx)
	{
		Constraint->SetModifierApplied();

		FVec3 WorldContactPoint0, WorldContactPoint1;
		GetWorldContactLocations(ContactPointIdx, WorldContactPoint0, WorldContactPoint1);

		TArrayView<FManifoldPoint> ManifoldPoints = Constraint->GetManifoldPoints();
		FManifoldPoint& ManifoldPoint = ManifoldPoints[ContactPointIdx];

		const FRigidTransform3& ShapeTransform1 = Constraint->GetShapeWorldTransform1();
		const FVec3 ShapeNormal = ShapeTransform1.InverseTransformVectorNoScale(Normal);

		ManifoldPoint.ContactPoint.ShapeContactNormal = ShapeNormal;
		ManifoldPoint.ContactPoint.Phi = FRealSingle(FVec3::DotProduct(WorldContactPoint0 - WorldContactPoint1, Normal));

		Modifier->MarkConstraintForManifoldUpdate(*Constraint);
	}

	void FContactPairModifier::GetWorldContactLocations(int32 ContactPointIdx, FVec3& OutLocation0, FVec3& OutLocation1) const
	{
		const FRigidTransform3& ShapeTransform0 = Constraint->GetShapeWorldTransform0();
		const FRigidTransform3& ShapeTransform1 = Constraint->GetShapeWorldTransform1();

		TArrayView<FManifoldPoint> ManifoldPoints = Constraint->GetManifoldPoints();
		const FManifoldPoint& ManifoldPoint = ManifoldPoints[ContactPointIdx];
		OutLocation0 = ShapeTransform0.TransformPositionNoScale(FVec3(ManifoldPoint.ContactPoint.ShapeContactPoints[0]));
		OutLocation1 = ShapeTransform1.TransformPositionNoScale(FVec3(ManifoldPoint.ContactPoint.ShapeContactPoints[1]));
	}

	FVec3 FContactPairModifier::GetWorldContactLocation(int32 ContactPointIdx) const
	{
		FVec3 WorldPos0, WorldPos1;
		GetWorldContactLocations(ContactPointIdx, WorldPos0, WorldPos1);
		return (WorldPos0 + WorldPos1) * FReal(0.5);
	}

	void FContactPairModifier::ModifyWorldContactLocations(const FVec3& Location0, const FVec3& Location1, int32 ContactPointIdx)
	{
		Constraint->SetModifierApplied();

		TArrayView<FManifoldPoint> ManifoldPoints = Constraint->GetManifoldPoints();
		FManifoldPoint& ManifoldPoint = ManifoldPoints[ContactPointIdx];

		ManifoldPoint.ContactPoint.ShapeContactPoints[0] = Constraint->GetShapeWorldTransform0().InverseTransformPositionNoScale(Location0);
		ManifoldPoint.ContactPoint.ShapeContactPoints[1] = Constraint->GetShapeWorldTransform1().InverseTransformPositionNoScale(Location1);

		// Clear the friction data since it will now have the wrong contact positions
		// @todo(chaos): maybe try to do something better here
		ManifoldPoint.ShapeAnchorPoints[0] = ManifoldPoint.ContactPoint.ShapeContactPoints[0];
		ManifoldPoint.ShapeAnchorPoints[1] = ManifoldPoint.ContactPoint.ShapeContactPoints[1];
		Constraint->ResetSavedManifoldPoints();

		Modifier->MarkConstraintForManifoldUpdate(*Constraint);
	}

	void FContactPairModifier::ModifyEnablePositionSolve(bool bEnable, int32 ContactPointIdx)
	{
		Constraint->SetManifoldPointPositionSolveEnabled(ContactPointIdx, bEnable);
		Constraint->SetModifierApplied();
	}

	void FContactPairModifier::ModifyEnableVelocitySolve(bool bEnable, int32 ContactPointIdx)
	{
		Constraint->SetManifoldPointVelocitySolveEnabled(ContactPointIdx, bEnable);
		Constraint->SetModifierApplied();
	}

	void FContactPairModifier::ModifyEnableFriction(bool bEnable, int32 ContactPointIdx)
	{
		Constraint->SetManifoldPointFrictionEnabled(ContactPointIdx, bEnable);
		Constraint->SetModifierApplied();
	}

	FReal FContactPairModifier::GetRestitution() const
	{
		return Constraint->GetRestitution();
	}

	void FContactPairModifier::ModifyRestitution(FReal Restitution)
	{
		Constraint->SetModifierApplied();

		Constraint->SetRestitution(Restitution);
	}

	FReal FContactPairModifier::GetRestitutionThreshold() const
	{
		return Constraint->GetRestitutionThreshold();
	}

	void FContactPairModifier::ModifyRestitutionThreshold(FReal Threshold)
	{
		Constraint->SetModifierApplied();

		Constraint->SetRestitutionThreshold(Threshold);
	}

	FReal FContactPairModifier::GetDynamicFriction() const
	{
		return Constraint->GetDynamicFriction();
	}

	void FContactPairModifier::ModifyDynamicFriction(FReal DynamicFriction)
	{
		Constraint->SetModifierApplied();

		Constraint->SetDynamicFriction(DynamicFriction);
	}

	FReal FContactPairModifier::GetStaticFriction() const
	{
		return Constraint->GetStaticFriction();
	}

	void FContactPairModifier::ModifyStaticFriction(FReal StaticFriction)
	{
		Constraint->SetModifierApplied();

		Constraint->SetStaticFriction(StaticFriction);
	}

	FVec3 FContactPairModifier::GetParticleVelocity(int32 ParticleIdx) const
	{
		const FGeometryParticleHandle* Particle = Constraint->GetParticle(ParticleIdx);
		const FKinematicGeometryParticleHandle* KinematicHandle = Particle->CastToKinematicParticle();
		if (!ensure(KinematicHandle))
		{
			// Cannot get velocity from static
			return FVec3(0);
		}

		return KinematicHandle->GetV();
	}

	void FContactPairModifier::ModifyParticleVelocity(FVec3 Velocity, int32 ParticleIdx)
	{
		Constraint->SetModifierApplied();

		FGeometryParticleHandle* Particle = Constraint->GetParticle(ParticleIdx);
		FKinematicGeometryParticleHandle* KinematicHandle = Particle->CastToKinematicParticle();
		if (!ensure(KinematicHandle))
		{
			// Cannot modify velocity on static
			return;
		}

		KinematicHandle->SetV(Velocity);

		// Simulated object must update implicit velocity
		if (FPBDRigidParticleHandle* RigidHandle = Particle->CastToRigidParticle())
		{
			EObjectStateType State(RigidHandle->ObjectState());
			if (State == EObjectStateType::Dynamic || State == EObjectStateType::Sleeping)
			{
				RigidHandle->SetX(RigidHandle->GetP() - Velocity * Modifier->Dt);
			}
		}
	}


	FVec3 FContactPairModifier::GetParticleAngularVelocity(int32 ParticleIdx) const
	{
		const FGeometryParticleHandle* Particle = Constraint->GetParticle(ParticleIdx);
		const FKinematicGeometryParticleHandle* KinematicHandle = Particle->CastToKinematicParticle();
		if (!ensure(KinematicHandle))
		{
			// Cannot get velocity from static
			return FVec3(0);
		}

		return KinematicHandle->GetW();
	}

	void FContactPairModifier::ModifyParticleAngularVelocity(FVec3 AngularVelocity, int32 ParticleIdx)
	{
		Constraint->SetModifierApplied();

		FGeometryParticleHandle* Particle = Constraint->GetParticle(ParticleIdx);
		FKinematicGeometryParticleHandle* KinematicHandle = Particle->CastToKinematicParticle();
		if (!ensure(KinematicHandle))
		{
			return;
		}

		KinematicHandle->SetW(AngularVelocity);

		// Simulated object must update implicit velocity
		if (FPBDRigidParticleHandle* RigidHandle = Particle->CastToRigidParticle())
		{
			EObjectStateType State(RigidHandle->ObjectState());
			if (State == EObjectStateType::Dynamic || State == EObjectStateType::Sleeping)
			{
				RigidHandle->SetR(FRotation3::IntegrateRotationWithAngularVelocity(RigidHandle->GetQ(), -RigidHandle->GetW(), Modifier->Dt));
			}
		}
	}

	FVec3 FContactPairModifier::GetParticlePosition(int32 ParticleIdx) const
	{
		const FGeometryParticleHandle* Particle = Constraint->GetParticle(ParticleIdx);
		const FPBDRigidParticleHandle* RigidHandle = Particle->CastToRigidParticle();

		if (RigidHandle)
		{
			return RigidHandle->GetP();
		}

		return Particle->GetX();
	}

	void FContactPairModifier::UpdateConstraintShapeTransforms()
	{
		const FRigidTransform3 ShapeWorldTransform0 = Constraint->GetShapeRelativeTransform0() * FConstGenericParticleHandle(Constraint->GetParticle0())->GetTransformPQ();
		const FRigidTransform3 ShapeWorldTransform1 = Constraint->GetShapeRelativeTransform1() * FConstGenericParticleHandle(Constraint->GetParticle1())->GetTransformPQ();
		Constraint->SetShapeWorldTransforms(ShapeWorldTransform0, ShapeWorldTransform1);
	}

	void FContactPairModifier::ModifyParticlePosition(FVec3 Position, bool bMaintainVelocity, int32 ParticleIdx)
	{
		Constraint->SetModifierApplied();

		FGeometryParticleHandle* Particle = Constraint->GetParticle(ParticleIdx);

		Modifier->MarkConstraintForManifoldUpdate(*Constraint);

		if (FPBDRigidParticleHandle* RigidHandle = Particle->CastToRigidParticle())
		{
			EObjectStateType State(RigidHandle->ObjectState());
			if (State == EObjectStateType::Dynamic || State == EObjectStateType::Sleeping)
			{
				RigidHandle->SetP(Position);

				if (bMaintainVelocity)
				{
					RigidHandle->SetX(RigidHandle->GetP() - RigidHandle->GetV() * Modifier->Dt);
				}
				else if(Modifier->Dt > 0.0f)
				{
					// Update V to new implicit velocity
					RigidHandle->SetV((RigidHandle->GetP() - RigidHandle->GetX()) / Modifier->Dt);
				}
				UpdateConstraintShapeTransforms();
				return;
			}
			else
			{
				// Kinematic must keep P/X in sync
				RigidHandle->SetX(Position);
				RigidHandle->SetP(Position);
				UpdateConstraintShapeTransforms();
				return;
			}
		}
		
		// Handle kinematic that is not PBDRigid type
		FKinematicGeometryParticleHandle* KinematicHandle = Particle->CastToKinematicParticle();
		if (KinematicHandle)
		{
			KinematicHandle->SetX(Position);
			UpdateConstraintShapeTransforms();
			return;
		}

		ensure(false); // Called on static?
	}


	FRotation3 FContactPairModifier::GetParticleRotation(int32 ParticleIdx) const
	{
		const FGeometryParticleHandle* Particle = Constraint->GetParticle(ParticleIdx);
		const FPBDRigidParticleHandle* RigidHandle = Particle->CastToRigidParticle();


		// We give predicted position for simulated objects
		if (RigidHandle)
		{
			return RigidHandle->GetQ();
		}

		return Particle->GetR();
	}

	void FContactPairModifier::ModifyParticleRotation(FRotation3 Rotation, bool bMaintainVelocity, int32 ParticleIdx)
	{
		Constraint->SetModifierApplied();

		FGeometryParticleHandle* Particle = Constraint->GetParticle(ParticleIdx);

		Modifier->MarkConstraintForManifoldUpdate(*Constraint);

		if (FPBDRigidParticleHandle* RigidHandle = Particle->CastToRigidParticle())
		{
			EObjectStateType State(RigidHandle->ObjectState());
			if (State == EObjectStateType::Dynamic || State == EObjectStateType::Sleeping)
			{
				RigidHandle->SetQ(Rotation);

				if (bMaintainVelocity)
				{
					RigidHandle->SetRf(FRotation3f::IntegrateRotationWithAngularVelocity(RigidHandle->GetQf(), -RigidHandle->GetWf(), FRealSingle(Modifier->Dt)));
				}
				else if (Modifier->Dt > 0.0f)
				{
					// Update W to new implicit velocity
					RigidHandle->SetWf(FRotation3f::CalculateAngularVelocity(RigidHandle->GetRf(), RigidHandle->GetQf(), FRealSingle(Modifier->Dt)));
				}
				UpdateConstraintShapeTransforms();
				return;
			}
			else
			{
				FRotation3f Rotationf(Rotation);
				// Kinematic must keep Q/R in sync
				RigidHandle->SetRf(Rotationf);
				RigidHandle->SetQf(Rotationf);
				UpdateConstraintShapeTransforms();
				return;
			}
		}
		// Handle kinematic that is not PBDRigid type
		FKinematicGeometryParticleHandle* KinematicHandle = Particle->CastToKinematicParticle();
		if (KinematicHandle)
		{
			KinematicHandle->SetR(Rotation);
			UpdateConstraintShapeTransforms();
			return;
		}

		ensure(false); // Called on static?
	}


	FReal FContactPairModifier::GetInvInertiaScale(int32 ParticleIdx) const
	{
		return ParticleIdx == 0 ? Constraint->GetInvInertiaScale0() : Constraint->GetInvInertiaScale1();
	}

	void FContactPairModifier::ModifyInvInertiaScale(FReal InInvInertiaScale, int32 ParticleIdx)
	{
		Constraint->SetModifierApplied();

		if (ParticleIdx == 0)
		{
			Constraint->SetInvInertiaScale0(InInvInertiaScale);
		}
		else
		{
			Constraint->SetInvInertiaScale1(InInvInertiaScale);
		}
	}

	FReal FContactPairModifier::GetInvMassScale(int32 ParticleIdx) const
	{
		return ParticleIdx == 0 ? Constraint->GetInvMassScale0() : Constraint->GetInvMassScale1();
	}

	void FContactPairModifier::ModifyInvMassScale(FReal InInvMassScale, int32 ParticleIdx)
	{
		Constraint->SetModifierApplied();

		if (ParticleIdx == 0)
		{
			Constraint->SetInvMassScale0(InInvMassScale);
		}
		else
		{
			Constraint->SetInvMassScale1(InInvMassScale);
		}
	}

	FGeometryParticleHandle* FContactPairModifier::GetOtherParticle(FGeometryParticleHandle* Particle) const
	{
		if (Particle == Constraint->GetParticle0())
		{
			return Constraint->GetParticle1();
		}
		else if (Particle == Constraint->GetParticle1())
		{
			return Constraint->GetParticle0();
		}
		return nullptr;
	}

	TVec2<FGeometryParticleHandle*> FContactPairModifier::GetParticlePair() const
	{
		return { Constraint->GetParticle0(), Constraint->GetParticle1() };
	}

	TVec2<const FPerShapeData*> FContactPairModifier::GetShapePair() const
	{
		return { Constraint->GetShape0(), Constraint->GetShape1() };
	}

	const FShapeInstance* FContactPairModifier::GetShape(int32 ParticleIdx) const
	{
		return Constraint->GetShape(ParticleIdx);
	}

	const FConstImplicitObjectRef FContactPairModifier::GetImplicit(int32 ParticleIdx) const
	{
		return Constraint->GetImplicit(ParticleIdx);
	}

	bool FContactPairModifier::GetIsCCD() const
	{
		return Constraint->GetCCDEnabled() && Constraint->GetCCDSweepEnabled();
	}


	bool FContactPairModifier::IsEdgeContactPoint(int32 ContactPointIdx) const
	{
		const FManifoldPoint& ManifoldPoint = Constraint->GetManifoldPoint(ContactPointIdx);

		return ManifoldPoint.ContactPoint.ContactType == EContactPointType::EdgeEdge;
	}

	bool FContactPairModifier::IsContactPointDisabled(int32 ContactPointIdx) const
	{
		const FManifoldPoint& ManifoldPoint = Constraint->GetManifoldPoint(ContactPointIdx);

		return ManifoldPoint.Flags.bDisabled;
	}

	void FContactPairModifier::SetContactPointDisabled(int32 ContactPointIdx) const
	{
		Constraint->DisableManifoldPoint(ContactPointIdx);
	}

	int32 FContactPairModifier::AddContact(const FVec3& WorldPos0, const FVec3& WorldPos1, const FVec3& WorldNormal) const
	{
		FContactPointf ContactPoint;
		ContactPoint.ShapeContactPoints[0] = Constraint->GetShapeWorldTransform0().InverseTransformPositionNoScale(WorldPos0);
		ContactPoint.ShapeContactPoints[1] = Constraint->GetShapeWorldTransform1().InverseTransformPositionNoScale(WorldPos1);
		ContactPoint.ShapeContactNormal = Constraint->GetShapeWorldTransform1().InverseTransformVectorNoScale(WorldNormal);
		ContactPoint.Phi = FRealSingle(FVec3::DotProduct(WorldPos0 - WorldPos1, WorldNormal));
	
		int32 ContactIndex = Constraint->AddManifoldPoint(ContactPoint);
		Constraint->SetModifierApplied();
		return ContactIndex;
	}


	const FPBDCollisionConstraintContainerCookie& FContactPairModifier::GetConstraintContainerCookie() const
	{
		return Constraint->GetContainerCookie();
	}

	void FContactPairModifierIterator::SeekValidContact()
	{
		// Not valid to call from end.
		if (!ensure(IsValid()))
		{
			return;
		}

		TArrayView<FPBDCollisionConstraint* const>& Constraints = Modifier->GetConstraints();

		while (ConstraintIdx < Constraints.Num())
		{
			FPBDCollisionConstraint* CurrentConstraint = Constraints[ConstraintIdx];


			TArrayView<FManifoldPoint> ManifoldPoints = CurrentConstraint->GetManifoldPoints();
			if (ManifoldPoints.Num())
			{
				PairModifier = FContactPairModifier(CurrentConstraint, *Modifier);
				return;
			}

			// Constraint has no points, try next constraint.
			++ConstraintIdx;
		}

		// No constraints remaining.
		SetToEnd();
	}

	FContactPairModifierParticleRangeIterator FContactPairModifierParticleRange::begin()
	{
		return FContactPairModifierParticleRangeIterator(Modifier, Constraints, 0);
	}

	FContactPairModifierParticleRangeIterator FContactPairModifierParticleRange::end()
	{
		return FContactPairModifierParticleRangeIterator(Modifier, Constraints, Constraints.Num());
	}

	FContactPairModifierParticleRange::FContactPairModifierParticleRange(FCollisionContactModifier* InModifier, FGeometryParticleHandle* InParticle)
		: Modifier(InModifier)
		, Particle(InParticle)
	{
		// NOTE: We only visit constraints that were activated this tick because we use the ActiveConstraintIndex to determine which
		// collisions we have already seen (see FVisitedContactPairsTracker::Visit) and this only works for collisions detected this tick.
		// We explicitly do not want to visit sleeping collisions.
		const ECollisionVisitorFlags VisitFlags = ECollisionVisitorFlags::VisitActiveAwake | ECollisionVisitorFlags::VisitDisabled;

		FParticleCollisions& ParticleCollisions = Particle->ParticleCollisions();
		ParticleCollisions.VisitCollisions([this](FPBDCollisionConstraint& Constraint)
		{
			if (Constraint.GetManifoldPoints().Num() > 0 && !Constraint.GetDisabled())
			{
				Constraints.Add(&Constraint);
			}
			return ECollisionVisitorResult::Continue;
		}, VisitFlags);
	}

	bool FVisitedContactPairsTracker::Visit(const FContactPairModifier& ContactPair)
	{
		// We use the index in the ActiveConstraints list as an ID for this constraint. This requires
		// that we are only visiting the active constraints (and not sleeping or expired constraints)
		// in the VisitCollisions call in FContactPairModifierParticleRange
		const int32 ConstraintIndex = ContactPair.GetConstraintContainerCookie().ConstraintIndex;
		
		// If we hit this, the constraint is not in the active list and the index is invalid
		// Likely an error in the VisitCollisions call above
		check(Constraints[ConstraintIndex] == ContactPair.Constraint);

		// Get a reference object to the relevant bit
		FBitReference BitRef = VisitedContacts[ConstraintIndex];

		// Mark constraint visited and return true
		if (BitRef == false)
		{
			BitRef = true;
			return true;
		}

		// Return false - this constraint has already been marked visited
		return false;
	}

	FContactPairModifierParticleRange FCollisionContactModifier::GetContacts(FGeometryParticleHandle* Particle)
	{
		return FContactPairModifierParticleRange(this, Particle);
	}

	FVisitedContactPairsTracker FCollisionContactModifier::MakeVisitedContactPairsTracker() const
	{
		return FVisitedContactPairsTracker(Constraints);
	}

	TArrayView<FPBDCollisionConstraint* const>& FCollisionContactModifier::GetConstraints()
	{
		return Constraints;
	}

	void FCollisionContactModifier::DisableConstraint(FPBDCollisionConstraint& Constraint)
	{
		Constraint.SetModifierApplied();

		Constraint.SetDisabled(true);
	}

	void FCollisionContactModifier::EnableConstraint(FPBDCollisionConstraint& Constraint)
	{
		Constraint.SetModifierApplied();

		Constraint.SetDisabled(false);
	}

	void FCollisionContactModifier::ConvertToProbeConstraint(FPBDCollisionConstraint& Constraint)
	{
		Constraint.SetModifierApplied();

		Constraint.SetIsProbe(true);
	}

	void FCollisionContactModifier::ConvertToNonProbeConstraint(FPBDCollisionConstraint& Constraint)
	{
		Constraint.SetModifierApplied();

		const FRigidTransform3& ShapeWorldTransform0 = Constraint.GetContact().GetShapeRelativeTransform0() * FParticleUtilities::GetActorWorldTransform(FConstGenericParticleHandle(Constraint.GetParticle0()));
		const FRigidTransform3& ShapeWorldTransform1 = Constraint.GetContact().GetShapeRelativeTransform1() * FParticleUtilities::GetActorWorldTransform(FConstGenericParticleHandle(Constraint.GetParticle1()));
		
		Constraint.SetShapeWorldTransforms(ShapeWorldTransform0, ShapeWorldTransform1);

		Collisions::UpdateConstraint(Constraint, ShapeWorldTransform0, ShapeWorldTransform1, Dt);

		Constraint.SetIsProbe(false);
	}

	void FCollisionContactModifier::MarkConstraintForManifoldUpdate(FPBDCollisionConstraint& Constraint)
	{
		NeedsManifoldUpdate.Add(&Constraint);
	}

	void FCollisionContactModifier::UpdateConstraintManifolds()
	{
		// Update derived state that depends on transforms etc
		for (FPBDCollisionConstraint* Constraint : NeedsManifoldUpdate)
		{
			Constraint->UpdateManifoldContacts();
		}

		NeedsManifoldUpdate.Reset();
	}
}