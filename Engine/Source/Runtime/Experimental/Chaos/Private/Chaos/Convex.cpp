// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/Convex.h"
#include "Chaos/GJK.h"
#include "Chaos/Sphere.h"
#include "Chaos/ImplicitObjectScaled.h"

#define CHAOS_CONVEX_USE_FAST_RAYCAST 1

namespace Chaos
{
	FImplicitObjectPtr FConvex::CopyGeometry() const
	{
		// const_cast required as this object has an intrusive reference count that need to be mutable
		return FImplicitObjectPtr(const_cast<FConvex*>(this));
	}

	FImplicitObjectPtr FConvex::CopyGeometryWithScale(const FVec3& Scale) const
	{
		// const_cast required as this object has an intrusive reference count that need to be mutable
		return FImplicitObjectPtr(new TImplicitObjectScaled<FConvex>(const_cast<FConvex*>(this), Scale));
	}

	FImplicitObjectPtr FConvex::DeepCopyGeometry() const
	{
		return FImplicitObjectPtr(new FConvex(*this));
	}

	FImplicitObjectPtr FConvex::DeepCopyGeometryWithScale(const FVec3& Scale) const
	{
		return FImplicitObjectPtr(new TImplicitObjectScaled<FConvex>(new FConvex(*this), Scale));
	}

	bool FConvex::Raycast(const FVec3& StartPoint, const FVec3& Dir, const FReal Length, const FReal Thickness, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex) const
	{
#if CHAOS_CONVEX_USE_FAST_RAYCAST
		// If we are just doing a raycast (Thickness == 0) we can run a faster direct ray test, but if it's
		// a sphere-cast we'll still need to defer to GJK for an accurate result
		if(Thickness == 0)
		{
			return RaycastFast(StartPoint, Dir, Length, Thickness, OutTime, OutPosition, OutNormal, OutFaceIndex);
		}
#endif

		OutFaceIndex = INDEX_NONE;	//finding face is expensive, should be called directly by user
		const FRigidTransform3 StartTM(StartPoint, FRotation3::FromIdentity());
		const FSphere Sphere(FVec3(0), Thickness);
		return GJKRaycast(*this, Sphere, StartTM, Dir, Length, OutTime, OutPosition, OutNormal);
	}

	bool FConvex::RaycastFast(const FVec3& StartPoint, const FVec3& Dir, const FReal Length, const FReal Thickness, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex) const
	{
		FReal EntryTime = 0.;
		FReal ExitTime = Length;
		int32 PlaneIndex = -1;
		
		for (int32 Idx = 0; Idx < Planes.Num(); ++Idx)
		{
			const FPlaneType& Plane = Planes[Idx];
			
			const FReal NDotDir = FVec3::DotProduct(Plane.Normal(), Dir);
			const FReal Distance = FVec3::DotProduct((StartPoint - FVec3{Plane.X()}), Plane.Normal()); //Plane.SignedDistance(StartPoint);

			if (NDotDir == 0.)
			{
				if (Distance > 0.)
				{
					// ray is perpendicular to one plane and outside of the convex, no intersection 
					return false;
				}
			}
			else
			{
				const FReal Time = -Distance /  NDotDir;
				if (NDotDir < 0.)
				{
					// ray opposite to the plane normal, it is an entry
					if (Time > EntryTime)
					{
						EntryTime = Time; 
						PlaneIndex = Idx;
					}
				}
				else
				{
					// ray has same orientation as the plane normal, it is an exit
					ExitTime = FMath::Min(ExitTime, Time);
				}
			}
			// early exit when possible
			if (EntryTime > ExitTime)
			{
				return false;
			};
		} 

		// successful intersection
		OutTime = EntryTime;
		OutPosition = StartPoint + (Dir * OutTime);
		OutNormal = (PlaneIndex >= 0)? FVec3{Planes[PlaneIndex].Normal()}: -Dir;
		OutFaceIndex = INDEX_NONE; // @todo(chaos) we could return PlaneIndex ( but would differ from previous implementation )
		return true;
	}

	int32 FConvex::FindMostOpposingFace(const FVec3& Position, const FVec3& UnitDir, int32 HintFaceIndex, FReal SearchDist) const
	{
		SearchDist = FMath::Max(SearchDist,(FReal)(FMath::Abs(BoundingBox().Extents().GetAbsMax()) * 1e-4f));
		//todo: use hill climbing
		int32 MostOpposingIdx = INDEX_NONE;
		FReal MostOpposingDot = TNumericLimits<FReal>::Max();
		for (int32 Idx = 0; Idx < Planes.Num(); ++Idx)
		{
			const FPlaneType& Plane = Planes[Idx];
			const FReal Distance = Plane.SignedDistance(Position);
			if (FMath::Abs(Distance) < SearchDist)
			{
				// TPlane has an override for Normal() that doesn't call PhiWithNormal().
				const FReal Dot = FVec3::DotProduct(Plane.Normal(), UnitDir);
				if (Dot < MostOpposingDot)
				{
					MostOpposingDot = Dot;
					MostOpposingIdx = Idx;
				}
			}
		}
		CHAOS_ENSURE(MostOpposingIdx != INDEX_NONE);
		return MostOpposingIdx;
	}

	// @todo(chaos): dedupe from above. Only difference is use of TPlaneConcrete<FReal, 3>::MakeScaledUnsafe
	int32 FConvex::FindMostOpposingFaceScaled(const FVec3& Position, const FVec3& UnitDir, int32 HintFaceIndex, FReal SearchDist, const FVec3& Scale) const
	{
		// Use of Scale.Max() is a bit dodgy, but the whole thing is fuzzy anyway
		SearchDist = FMath::Max(Scale.Max() * SearchDist, FMath::Abs(BoundingBox().Extents().GetAbsMax()) * 1e-4f);

		//todo: use hill climbing
		int32 MostOpposingIdx = INDEX_NONE;
		FReal MostOpposingDot = TNumericLimits<FReal>::Max();
		for (int32 Idx = 0; Idx < Planes.Num(); ++Idx)
		{
			// @todo(chaos): pass in InvScale
			const FPlaneType Plane = FPlaneType::MakeScaledUnsafe(Planes[Idx], Scale, FVec3(1) / Scale);
			const FReal Distance = Plane.SignedDistance(Position);
			if (FMath::Abs(Distance) < SearchDist)
			{
				// TPlane has an override for Normal() that doesn't call PhiWithNormal().
				const FReal Dot = FVec3::DotProduct(Plane.Normal(), UnitDir);
				if (Dot < MostOpposingDot)
				{
					MostOpposingDot = Dot;
					MostOpposingIdx = Idx;
				}
			}
		}
		CHAOS_ENSURE(MostOpposingIdx != INDEX_NONE);
		return MostOpposingIdx;
	}


	int32 FConvex::FindClosestFaceAndVertices(const FVec3& Position, TArray<FVec3>& FaceVertices, FReal SearchDist) const
	{
		//
		//  @todo(chaos) : Collision Manifold 
		//     Create a correspondence between the faces and surface particles on construction.
		//     The correspondence will provide an index between the Planes and the Vertices,
		//     removing the need for the exhaustive search here. 
		//

		int32 ReturnIndex = INDEX_NONE;
		TSet<int32> IncludedParticles;
		for (int32 Idx = 0; Idx < Planes.Num(); ++Idx)
		{
			const FPlaneType& Plane = Planes[Idx];
			FReal AbsOfSignedDistance = FMath::Abs(Plane.SignedDistance(Position));
			if (AbsOfSignedDistance < SearchDist)
			{
				for (int32 Fdx = 0; Fdx < (int32)Vertices.Num(); Fdx++)
				{
					if (!IncludedParticles.Contains(Fdx))
					{
						if (FMath::Abs(Plane.SignedDistance(Vertices[Fdx])) < SearchDist)
						{
							FaceVertices.Add(Vertices[Fdx]);
							IncludedParticles.Add(Fdx);
						}
					}
				}
				ReturnIndex = Idx;
			}
		}
		return ReturnIndex;
	}

	int32 FConvex::GetMostOpposingPlane(const FVec3& Normal) const
	{
		// @todo(chaos): use hill climbing
		int32 MostOpposingIdx = INDEX_NONE;
		FReal MostOpposingDot = TNumericLimits<FReal>::Max();
		for (int32 Idx = 0; Idx < Planes.Num(); ++Idx)
		{
			const FPlaneType& Plane = Planes[Idx];
			const FReal Dot = FVec3::DotProduct(Plane.Normal(), Normal);
			if (Dot < MostOpposingDot)
			{
				MostOpposingDot = Dot;
				MostOpposingIdx = Idx;
			}
		}
		CHAOS_ENSURE(MostOpposingIdx != INDEX_NONE);
		return MostOpposingIdx;
	}

	int32 FConvex::GetMostOpposingPlaneScaled(const FVec3& Normal, const FVec3& Scale) const
	{
		// NOTE: We cannot just call the scale-less version like we can for box, even if we unscale the normal
		// @todo(chaos): use hill climbing
		int32 MostOpposingIdx = INDEX_NONE;
		FReal MostOpposingDot = TNumericLimits<FReal>::Max();
		for (int32 Idx = 0; Idx < Planes.Num(); ++Idx)
		{
			const FPlaneType& Plane = Planes[Idx];
			const FVec3 ScaledNormal = (Plane.Normal() / Scale).GetSafeNormal();
			const FReal Dot = FVec3::DotProduct(ScaledNormal, Normal);
			if (Dot < MostOpposingDot)
			{
				MostOpposingDot = Dot;
				MostOpposingIdx = Idx;
			}
		}
		CHAOS_ENSURE(MostOpposingIdx != INDEX_NONE);
		return MostOpposingIdx;
	}

	FVec3 FConvex::GetClosestEdge(int32 PlaneIndex, const FVec3& InPosition, FVec3& OutEdgePos0, FVec3& OutEdgePos1) const
	{
		FVec3Type ClosestEdgePosition = FVec3Type(0);
		FRealType ClosestDistanceSq = FLT_MAX;
		FVec3Type Position = FVec3Type(InPosition);

		const int32 PlaneVerticesNum = NumPlaneVertices(PlaneIndex);
		if (PlaneVerticesNum > 0)
		{
			FVec3Type P0 = GetVertex(GetPlaneVertex(PlaneIndex, PlaneVerticesNum - 1));
			for (int32 PlaneVertexIndex = 0; PlaneVertexIndex < PlaneVerticesNum; ++PlaneVertexIndex)
			{
				const int32 VertexIndex = GetPlaneVertex(PlaneIndex, PlaneVertexIndex);
				const FVec3Type P1 = GetVertex(VertexIndex);
				
				// LWC_TODO: low-precision version of FMath::ClosestPointOnLine
				// See FMath::ClosestPointOnLine (which does not have a float version when LWC is enbled)
				//const FVec3Type EdgePosition = FMath::ClosestPointOnLine(P0, P1, Position);
				const FVec3Type DP = P1 - P0;
				const FRealType A = FVec3Type::DotProduct((P0 - Position), DP);
				const FRealType B = DP.SizeSquared();
				const FRealType T = FMath::Clamp(-A / B, FRealType(0), FRealType(1));
				const FVec3Type EdgePosition = P0 + (T * DP);
				
				const FRealType EdgeDistanceSq = (EdgePosition - Position).SizeSquared();
				if (EdgeDistanceSq < ClosestDistanceSq)
				{
					ClosestDistanceSq = EdgeDistanceSq;
					ClosestEdgePosition = EdgePosition;
					OutEdgePos0 = P0;
					OutEdgePos1 = P1;
				}

				P0 = P1;
			}
		}

		return FVec3(ClosestEdgePosition);
	}

	bool FConvex::GetClosestEdgeVertices(int32 PlaneIndex, const FVec3& Position, int32& OutVertexIndex0, int32& OutVertexIndex1) const
	{
		OutVertexIndex0 = INDEX_NONE;
		OutVertexIndex1 = INDEX_NONE;

		FReal ClosestDistanceSq = FLT_MAX;
		const int32 PlaneVerticesNum = NumPlaneVertices(PlaneIndex);
		if (PlaneVerticesNum > 0)
		{
			int32 VertexIndex0 = GetPlaneVertex(PlaneIndex, PlaneVerticesNum - 1);
			FVec3 P0 = GetVertex(VertexIndex0);

			for (int32 PlaneVertexIndex = 0; PlaneVertexIndex < PlaneVerticesNum; ++PlaneVertexIndex)
			{
				const int32 VertexIndex1 = GetPlaneVertex(PlaneIndex, PlaneVertexIndex);
				const FVec3 P1 = GetVertex(VertexIndex1);

				const FVec3 EdgePosition = FMath::ClosestPointOnLine(P0, P1, Position);
				const FReal EdgeDistanceSq = (EdgePosition - Position).SizeSquared();

				if (EdgeDistanceSq < ClosestDistanceSq)
				{
					OutVertexIndex0 = VertexIndex0;
					OutVertexIndex1 = VertexIndex1;
					ClosestDistanceSq = EdgeDistanceSq;
				}

				VertexIndex0 = VertexIndex1;
				P0 = P1;
			}
			return true;
		}
		return false;
	}

	int32 FConvex::FindVertexPlanes(int32 VertexIndex, int32* OutVertexPlanes, int32 MaxVertexPlanes) const
	{
		if (StructureData.IsValid())
		{
			return StructureData.FindVertexPlanes(VertexIndex, OutVertexPlanes, MaxVertexPlanes);
		}
		return 0;
	}

	int32 FConvex::GetVertexPlanes3(int32 VertexIndex, int32& PlaneIndex0, int32& PlaneIndex1, int32& PlaneIndex2) const
	{
		if (StructureData.IsValid())
		{
			return StructureData.GetVertexPlanes3(VertexIndex, PlaneIndex0, PlaneIndex1, PlaneIndex2);
		}
		return 0;
	}

	// Store the structure data with the convex. This is used by manifold generation, for example
	DECLARE_CYCLE_STAT(TEXT("FConvex::CreateStructureData"), STAT_CreateConvexStructureData, STATGROUP_ChaosCollision);
	void FConvex::CreateStructureData(TArray<TArray<int32>>&& PlaneVertexIndices, const bool bRegularDatas)
	{
		SCOPE_CYCLE_COUNTER(STAT_CreateConvexStructureData);
		const bool bSuccess = StructureData.SetPlaneVertices(MoveTemp(PlaneVertexIndices), Vertices.Num(), bRegularDatas);
		if (!bSuccess || !StructureData.IsValid())
		{
			UE_LOGF(LogChaos, Error, "Unable to create structure data for %ls", *ToStringFull());
		}
	}

	void FConvex::MovePlanesAndRebuild(const FRealType InDelta)
	{
		TArray<FPlane4f> NewPlanes;
		TArray<FVec3Type> NewPoints;
		NewPlanes.Reserve(Planes.Num());

		// Move all the planes inwards
		for (int32 PlaneIndex = 0; PlaneIndex < Planes.Num(); ++PlaneIndex)
		{
			const FPlane4f NewPlane = FPlane4f(Planes[PlaneIndex].X() + InDelta * Planes[PlaneIndex].Normal(), Planes[PlaneIndex].Normal());
			NewPlanes.Add(NewPlane);
		}

		// Recalculate the set of points from the intersection of all combinations of 3 planes
		// There will be NC3 of these (N! / (3! * (N-3)!)
		const FRealType PointTolerance = 1e-2f;
		for (int32 PlaneIndex0 = 0; PlaneIndex0 < NewPlanes.Num(); ++PlaneIndex0)
		{
			for (int32 PlaneIndex1 = PlaneIndex0 + 1; PlaneIndex1 < NewPlanes.Num(); ++PlaneIndex1)
			{
				for (int32 PlaneIndex2 = PlaneIndex1 + 1; PlaneIndex2 < NewPlanes.Num(); ++PlaneIndex2)
				{
					FVec3Type PlanesPos;
					if (FMath::IntersectPlanes3<FRealType>(PlanesPos, NewPlanes[PlaneIndex0], NewPlanes[PlaneIndex1], NewPlanes[PlaneIndex2]))
					{
						// Reject duplicate points
						int32 NewPointIndex = INDEX_NONE;
						for (int32 PointIndex = 0; PointIndex < NewPoints.Num(); ++PointIndex)
						{
							if ((PlanesPos - NewPoints[PointIndex]).SizeSquared() < PointTolerance * PointTolerance)
							{
								NewPointIndex = PointIndex;
								break;
							}
						}
						if (NewPointIndex == INDEX_NONE)
						{
							NewPointIndex = NewPoints.Add(PlanesPos);
						}
					}
				}
			}
		}

		// Reject points outside the planes to get down to a sensible number for the build step
		const FRealType PointPlaneTolerance = PointTolerance;
		int32 NumNewPoints = NewPoints.Num();
		for (int32 PointIndex = 0; PointIndex < NumNewPoints; ++PointIndex)
		{
			const int32 NumNewPlanes = NewPlanes.Num();
			for (int32 PlaneIndex = 0; PlaneIndex < NumNewPlanes; ++PlaneIndex)
			{
				const FRealType PointPlaneDistance = NewPlanes[PlaneIndex].PlaneDot(NewPoints[PointIndex]);
				if (PointPlaneDistance > PointPlaneTolerance)
				{
					NewPoints.RemoveAtSwap(PointIndex);
					--PointIndex;
					--NumNewPoints;
					break;
				}
			}
		}

		// Generate a new convex from the points
		*this = FConvex(NewPoints, 0.0f);
	}

	DECLARE_CYCLE_STAT(TEXT("FConvex::ComputeUnitMassInertiaTensorAndRotationOfMass"), STAT_ComputeConvexMassInertia, STATGROUP_ChaosCollision);
	void FConvex::ComputeUnitMassInertiaTensorAndRotationOfMass(const FReal InVolume)
	{
		SCOPE_CYCLE_COUNTER(STAT_ComputeConvexMassInertia);
		if (InVolume < UE_SMALL_NUMBER || !StructureData.IsValid())
		{
			UnitMassInertiaTensor = FVec3{1., 1., 1.};
			RotationOfMass = FRotation3::Identity;
			return;
		}

		// we only compute the inertia tensor using unit mass and scale it upon request
		constexpr FReal Mass = 1.0;
		const FReal Density = Mass / InVolume;

		static const FMatrix33 Standard(2, 1, 1, 2, 1, 2);
		FMatrix33 Covariance(0);

		for (int32 PlaneIndex = 0; PlaneIndex < Planes.Num(); ++PlaneIndex)
		{
			const int32 PlaneVerticesNum = NumPlaneVertices(PlaneIndex);
			if (PlaneVerticesNum > 2)
			{
				// compute plane center
				FVec3 PlaneCenter(0);
				for (int32 VertexIndex = 0; VertexIndex < PlaneVerticesNum; VertexIndex++)
				{
					PlaneCenter += FVec3(GetVertex(GetPlaneVertex(PlaneIndex, VertexIndex)));
				}
				PlaneCenter /= PlaneVerticesNum;

				// Now break down the plane in triangle ( fan around PlaneCenter ) 
				FMatrix33 DeltaMatrix(0);
				for (int32 VertexIndex = 0; VertexIndex < PlaneVerticesNum; VertexIndex++)
				{
					FVec3 DeltaVector = PlaneCenter - CenterOfMass;
					DeltaMatrix.M[0][0] = DeltaVector[0];
					DeltaMatrix.M[1][0] = DeltaVector[1];
					DeltaMatrix.M[2][0] = DeltaVector[2];

					DeltaVector = GetVertex(GetPlaneVertex(PlaneIndex, VertexIndex)) - CenterOfMass;
					DeltaMatrix.M[0][1] = DeltaVector[0];
					DeltaMatrix.M[1][1] = DeltaVector[1];
					DeltaMatrix.M[2][1] = DeltaVector[2];

					DeltaVector = GetVertex(GetPlaneVertex(PlaneIndex, (VertexIndex + 1) % PlaneVerticesNum)) - CenterOfMass;
					DeltaMatrix.M[0][2] = DeltaVector[0];
					DeltaMatrix.M[1][2] = DeltaVector[1];
					DeltaMatrix.M[2][2] = DeltaVector[2];

					FReal Det = DeltaMatrix.M[0][0] * (DeltaMatrix.M[1][1] * DeltaMatrix.M[2][2] - DeltaMatrix.M[1][2] * DeltaMatrix.M[2][1]) -
						DeltaMatrix.M[0][1] * (DeltaMatrix.M[1][0] * DeltaMatrix.M[2][2] - DeltaMatrix.M[1][2] * DeltaMatrix.M[2][0]) +
						DeltaMatrix.M[0][2] * (DeltaMatrix.M[1][0] * DeltaMatrix.M[2][1] - DeltaMatrix.M[1][1] * DeltaMatrix.M[2][0]);
					const FMatrix33 ScaledStandard = Standard * Det;
					Covariance += DeltaMatrix * ScaledStandard * DeltaMatrix.GetTransposed();
				}
			}
		}
		FReal Trace = Covariance.M[0][0] + Covariance.M[1][1] + Covariance.M[2][2];
		FMatrix33 TraceMat(Trace, Trace, Trace);
		FMatrix33 InertiaTensor = (TraceMat - Covariance) * (1 / (FReal)120) * Density;
		RotationOfMass = TransformToLocalSpace(InertiaTensor);
		UnitMassInertiaTensor = InertiaTensor.GetDiagonal();
	}
}