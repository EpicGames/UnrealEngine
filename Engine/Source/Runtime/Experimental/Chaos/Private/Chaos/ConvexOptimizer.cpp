// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/ConvexOptimizer.h"
#include "Chaos/Tribox.h"
#include "ChaosStats.h"
#include "Chaos/ImplicitObjectBVH.h"
#include "Chaos/ImplicitObjectUnion.h"
#include "Chaos/PhysicsObjectCollisionInterface.h"

namespace Chaos
{
namespace CVars
{
	// Replace all the convexes within an implicit hierarchy with a simplified one (kdop18 tribox for now) for collision
	CHAOS_API bool bChaosConvexSimplifyUnion = false;
	FAutoConsoleVariableRef CVarChaosSimplifyUnion(TEXT("p.Chaos.Convex.SimplifyUnion"), bChaosConvexSimplifyUnion, TEXT("If true replace all the convexes within an implcit hierarchy with a simplified one (kdop18 tribox for now) for collision"));

	// Max number of convex LODs used during simplification  for dynamic particles
	int32 ChaosConvexKinematicMode = 2;
	FAutoConsoleVariableRef CVarChaosConvexKinematicMode(TEXT("p.Chaos.Convex.KinematicMode"), ChaosConvexKinematicMode, TEXT("Simplification mode for the kinematic shapes (0: Single Convex, 1: One convex per children, 2: Merge connected children using the splitting threshold"));

	// Max number of convex LODs used during simplification  for kinematic particles
	int32 ChaosConvexDynamicMode = 2;
	FAutoConsoleVariableRef CVarChaosConvexDynamicMode(TEXT("p.Chaos.Convex.DynamicMode"), ChaosConvexDynamicMode, TEXT("Simplification mode for the dynamic shapes (0: Single Convex, 1: One convex per children, 2: Merge connected children using the splitting threshold)"));
	
	// Tribox volume / convex hull threshold to trigger a volume splitting during tree construction
	float ChaosConvexSplittingThreshold = 1.0f;
	FAutoConsoleVariableRef CVarChaosConvexSplittingThreshold(TEXT("p.Chaos.Convex.SplittingThreshold"), ChaosConvexSplittingThreshold, TEXT("Tribox volume / convex hull threshold to trigger a volume splitting during tree construction"));

	// Min volume of the simplified convexes
	float ChaosConvexMinVolume = 10000.0f;
	FAutoConsoleVariableRef CVarChaosConvexMinVolume(TEXT("p.Chaos.Convex.MinVolume"), ChaosConvexMinVolume, TEXT("Min volume of the simplified convexes"));

	// Boolean to check if we are merging (bottom-up) or splitting (top-bottom) the convexes 
	bool ChaosConvexEnableMerging = true;
	FAutoConsoleVariableRef CVarChaosConvexEnableMerging(TEXT("p.Chaos.Convex.EnableMerging"), ChaosConvexEnableMerging, TEXT("Boolean to check if we are merging (bottom-up) or splitting (top-bottom) the convexes"));
	
	// Boolean to check if we can use direct overlap test against the bounds (it assumes we only have leaves)
	bool ChaosConvexUseDirectOverlap = true;
	FAutoConsoleVariableRef CVarChaosConvexUseDirectOverlap(TEXT("p.Chaos.Convex.UseDirectOverlap"), ChaosConvexUseDirectOverlap, TEXT("Boolean to check if we can use direct overlap test against the bounds"));
	
	extern int32 ChaosUnionBVHMaxDepth;
	extern int32 ChaosUnionBVHMinShapes;
}

namespace Private
{

FConvexOptimizer::FConvexOptimizer() :
	SimplifiedConvexes(), CollisionObjects(MakeUnique<Private::FCollisionObjects>()), ShapesArray()
{}

FConvexOptimizer::~FConvexOptimizer() = default;

void FConvexOptimizer::VisitCollisionObjects(const FImplicitHierarchyVisitor& VisitorFunc) const
{
	if(CVars::bChaosConvexSimplifyUnion)
	{
		const TArray<Private::FImplicitBVHObject>& ImplicitObjects = BVH.IsValid() ? BVH->GetObjects() : CollisionObjects->ImplicitObjects;
		
		int32 ObjectIndex = 1;
		for(const Private::FImplicitBVHObject& CollisionObject : ImplicitObjects)
		{
			int32 LocalLeafObjectIndex = CollisionObject.GetObjectIndex();
			CollisionObject.GetGeometry()->VisitLeafObjectsImpl(CollisionObject.GetTransform(),
				CollisionObject.GetRootObjectIndex(), ObjectIndex, LocalLeafObjectIndex, VisitorFunc);
		}
	}	
}

void FConvexOptimizer::VisitOverlappingObjects(const FAABB3& LocalBounds, const FImplicitHierarchyVisitor& VisitorFunc) const
{
	if(CVars::bChaosConvexSimplifyUnion)
	{
		if(CVars::ChaosConvexUseDirectOverlap)
		{
			const FAABB3f CollisionBounds(LocalBounds);
			int32 ObjectIndex = 1;
			if(BVH.IsValid())
			{
				BVH->VisitAllIntersections(LocalBounds,
				[this, &ObjectIndex, &VisitorFunc,&CollisionBounds](const FImplicitObject* Implicit, const FRigidTransform3f& RelativeTransformf,
					const FAABB3f& RelativeBoundsf, const int32 RootObjectIndex, const int32 LeafObjectIndex)
				{
					if(CollisionBounds.Intersects(RelativeBoundsf))
					{
						VisitorFunc(Implicit, FRigidTransform3(RelativeTransformf), RootObjectIndex, ObjectIndex++, LeafObjectIndex);
					}
				});
			}
			else
			{
				for(const Private::FImplicitBVHObject& CollisionObject : CollisionObjects->ImplicitObjects)
				{
					if(CollisionBounds.Intersects(CollisionObject.GetBounds()))
					{
						VisitorFunc(CollisionObject.GetGeometry(), CollisionObject.GetTransform(), CollisionObject.GetRootObjectIndex(), ObjectIndex++, CollisionObject.GetObjectIndex());
					}
				}
			}
		}
		else
		{
			//TODO: remove this version and the cvar since the LocalBounds is in object space and the
			// overlap test done in VisitOverlappingLeafObjectsImpl assume a LocalBounds in implicit space
			int32 ObjectIndex = 1;
			if(BVH.IsValid())
			{
				BVH->VisitAllIntersections(LocalBounds,
				[this, &ObjectIndex, &VisitorFunc, &LocalBounds](const FImplicitObject* Implicit, const FRigidTransform3f& RelativeTransformf,
					const FAABB3f& RelativeBoundsf, const int32 RootObjectIndex, const int32 LeafObjectIndex)
				{
					int32 LocalLeafObjectIndex = LeafObjectIndex;
					Implicit->VisitOverlappingLeafObjectsImpl(LocalBounds, FRigidTransform3(RelativeTransformf),
						RootObjectIndex, ObjectIndex, LocalLeafObjectIndex, VisitorFunc);
				});
			}
			else
			{
				for(const Private::FImplicitBVHObject& CollisionObject : CollisionObjects->ImplicitObjects)
				{
					int32 LocalLeafObjectIndex = CollisionObject.GetObjectIndex();
					CollisionObject.GetGeometry()->VisitOverlappingLeafObjectsImpl(LocalBounds, CollisionObject.GetTransform(),
						CollisionObject.GetRootObjectIndex(), ObjectIndex, LocalLeafObjectIndex, VisitorFunc);
				}
			}
		}
	}
}

DECLARE_CYCLE_STAT(TEXT("Collisions::BuildConvexShapes"), STAT_BuildConvexShapes, STATGROUP_ChaosCollision);
void FConvexOptimizer::BuildConvexShapes(const FShapesArray& UnionShapes)
{
	SCOPE_CYCLE_COUNTER(STAT_BuildConvexShapes);
	
	ShapesArray.Reset();
	TArray<FImplicitObjectPtr> ImplicitObjects = SimplifiedConvexes;
	ShapesArray.Add(FShapeInstance::Make(UnionShapes.Num(),
		MakeImplicitObjectPtr<FImplicitObjectUnion>(MoveTemp(ImplicitObjects))));

	const TUniquePtr<FPerShapeData>& TemplateData = UnionShapes[0];
	const TUniquePtr<FShapeInstance>& ShapeData = ShapesArray.Last();

	ShapeData->SetCombinedShapeFilterData(TemplateData->GetCombinedShapeFilterData());
	ShapeData->SetCollisionTraceType(TemplateData->GetCollisionTraceType());

	// Only simulation enabled if all the underlying shapes could be used for physics
	ShapeData->SetSimEnabled(true);

	// Disable simple shapes for query 
	ShapeData->SetQueryEnabled(false);
}

void FConvexOptimizer::SimplifyRootConvexes(const Chaos::FImplicitObjectUnionPtr& UnionGeometry, const FShapesArray& UnionShapes,
		const EObjectStateType ObjectState, const TBitArray<>& bOptimizeConvexes)
{
	SCOPE_CYCLE_COUNTER(STAT_Collisions_SimplifyConvexes);

	if(CVars::bChaosConvexSimplifyUnion && UnionGeometry && (UnionShapes.Num() > 0) && (UnionShapes.Num() == bOptimizeConvexes.Num()) &&
		(UnionShapes.Num() <= UnionGeometry->GetObjects().Num()))
	{
		SimplifiedConvexes.Reset();
		CollisionObjects->ImplicitObjects.Reset();
		ShapesArray.Reset();

		if((UnionShapes.Num() == 1) || ((ObjectState == EObjectStateType::Dynamic) && (CVars::ChaosConvexDynamicMode == 0)) 
				|| ((ObjectState != EObjectStateType::Dynamic) && (CVars::ChaosConvexKinematicMode == 0)))
		{
			BuildSingleConvex(UnionGeometry, UnionShapes, bOptimizeConvexes);
		}
		else
		{
			const bool bEnableMerging = ((ObjectState == EObjectStateType::Dynamic) && (CVars::ChaosConvexDynamicMode == 2))
				|| ((ObjectState != EObjectStateType::Dynamic) && (CVars::ChaosConvexKinematicMode == 2));
			BuildMultipleConvex(UnionGeometry, UnionShapes, bEnableMerging, bOptimizeConvexes);
		}
	}

	if(CVars::bChaosUnionBVHEnabled)
	{
		if(BVH.IsValid())
		{
			BVH.Reset();
		}
		BVH = FImplicitBVH::TryMakeFromLeaves(MoveTemp(CollisionObjects->ImplicitObjects), CVars::ChaosUnionBVHMinShapes, CVars::ChaosUnionBVHMaxDepth);
	}
}

FORCEINLINE void InvalidateCachedTriboxes(FConvexOptimizer::FTriboxNodes& RootTriboxes)
{
	for(auto& RootTribox : RootTriboxes)
	{
		RootTribox.Value.NodeTribox.SetValid(false);
	}
}

DECLARE_CYCLE_STAT(TEXT("Collisions::ResizeCachedTriboxes"), STAT_ResizeCachedTriboxes, STATGROUP_ChaosCollision);
FORCEINLINE void ResizeCachedTriboxes(FConvexOptimizer::FTriboxNodes& RootTriboxes)
{
	SCOPE_CYCLE_COUNTER(STAT_ResizeCachedTriboxes);

	TArray<FImplicitObject*> NodesToRemove;
	TArray<FImplicitObject*> EdgesToRemove;
	for(auto& RootTribox : RootTriboxes)
	{
		if(!RootTribox.Value.NodeTribox.IsValid())
		{
			NodesToRemove.Add(RootTribox.Key);
		}
		else
		{
			EdgesToRemove.Reset();
			for(auto& NodeEdge : RootTribox.Value.bValidEdges)
			{
				if(auto* EdgeNode = RootTriboxes.Find(NodeEdge.Key))
				{
					if(!EdgeNode->NodeTribox.IsValid())
					{
						EdgesToRemove.Add(NodeEdge.Key);
					}
				}
			}
			for(auto& KeyToRemove : EdgesToRemove)
			{
				RootTribox.Value.bValidEdges.Remove(KeyToRemove);
			}
		}
	}
	for(auto& KeyToRemove : NodesToRemove)
	{
		RootTriboxes.Remove(KeyToRemove);
	}
}

DECLARE_CYCLE_STAT(TEXT("Collisions::BuildConvexTriboxes"), STAT_BuildConvexTriboxes, STATGROUP_ChaosCollision);
FORCEINLINE void BuildConvexTriboxes(const Chaos::FImplicitObjectUnionPtr& UnionGeometry, const FShapesArray& UnionShapes,
	TUniquePtr<Private::FCollisionObjects>& CollisionObjects, FConvexOptimizer::FTriboxNodes& RootTriboxes, int32& NextConvexId, const TBitArray<>& bOptimizeConvexes)
{
	SCOPE_CYCLE_COUNTER(STAT_BuildConvexTriboxes);

	// Invalidate the cached root triboxes
	InvalidateCachedTriboxes(RootTriboxes);
	
	for(int32 RootObjectIndex = 0, NumRootObjects = UnionGeometry->GetObjects().Num(); RootObjectIndex < NumRootObjects; ++RootObjectIndex)
	{
		const int32 ShapeIndex = UnionShapes.IsValidIndex(RootObjectIndex) ? RootObjectIndex : 0;
		if(UnionShapes[ShapeIndex]->GetSimEnabled())
		{
			if(FImplicitObject* RootObject = UnionGeometry->GetObjects()[RootObjectIndex].GetReference())
			{
				FConvexOptimizer::FTriboxNode* RootTribox = RootTriboxes.Find(RootObject);
				const bool bHasRootTribox = (RootTribox != nullptr);
				if(bHasRootTribox) RootTribox->bValidNode = false;
				
				FTribox LocalTribox;
				int32 LeafObjectIndex = 0, ObjectIndex = 0;
				RootObject->VisitLeafObjectsImpl(FRigidTransform3::Identity,RootObjectIndex, ObjectIndex, LeafObjectIndex,
					[&CollisionObjects, &LocalTribox, &bHasRootTribox, &ShapeIndex, &NextConvexId, &bOptimizeConvexes](
					const FImplicitObject* ImplicitObject, const FRigidTransform3& RelativeTransform,
					const int32 RootIndex, const int32, const int32)
				{
					if (!bOptimizeConvexes[ShapeIndex])
					{
						// Add non convex implicits to the list of collision objects
						Private::FImplicitBVH::CollectLeafObject(ImplicitObject,
							RelativeTransform, ShapeIndex, CollisionObjects->ImplicitObjects, NextConvexId++);
					}
					else
					{ 
						if(const FConvex* Convex = ImplicitObject->AsA<FConvex>())
						{
							if(!bHasRootTribox)
							{
								const FTribox::FRigidTransform3Type ConvexTransform(RelativeTransform);
								// Disable collision for all the convexes that are going to be used to build the tribox
								// For now only used for debug draw
								LocalTribox.AddConvex(Convex, ConvexTransform);
							}
						}
						else if(const TImplicitObjectScaled<FConvex>* ScaledObject = TImplicitObjectScaled<FConvex>::AsScaled(*ImplicitObject))
						{
							if(!bHasRootTribox)
							{
								const FTribox::FRigidTransform3Type ConvexTransform(RelativeTransform.GetTranslation(),
									RelativeTransform.GetRotation(), RelativeTransform.GetScale3D() * ScaledObject->GetScale());
								// Disable collision for all the convexes that are going to be used to build the tribox
								// For now only used for debug draw
								LocalTribox.AddConvex(ScaledObject->GetUnscaledObject(), ConvexTransform);
							}
						}
						else if(const TImplicitObjectInstanced<FConvex>* InstancedObject = TImplicitObjectInstanced<FConvex>::AsInstanced(*ImplicitObject))
						{
							if(!bHasRootTribox)
							{
								const FTribox::FRigidTransform3Type ConvexTransform(RelativeTransform);
								// Disable collision for all the convexes that are going to be used to build the tribox
								// For now only used for debug draw
								LocalTribox.AddConvex(InstancedObject->Object(), ConvexTransform);
							}
						}
						else
						{
							// Add non convex implicits to the list of collision objects
							Private::FImplicitBVH::CollectLeafObject(ImplicitObject,
								RelativeTransform, ShapeIndex, CollisionObjects->ImplicitObjects, NextConvexId++);
						}
					}
				});

				if (!bHasRootTribox && LocalTribox.HasDatas())
				{
					LocalTribox.BuildTribox();
					RootTribox = &(RootTriboxes.Add(RootObject, { LocalTribox, nullptr, ShapeIndex, LocalTribox.ComputeVolume(),true }));
				}
				if (RootTribox)
				{
					RootTribox->NodeTribox.SetValid(true);
					RootTribox->ShapeIndex = ShapeIndex;
				}
			}
		}
	}
	ResizeCachedTriboxes(RootTriboxes);
}

void FConvexOptimizer::BuildSingleConvex(const Chaos::FImplicitObjectUnionPtr& UnionGeometry, const FShapesArray& UnionShapes, const TBitArray<>& bOptimizeConvexes)
{
	BuildConvexTriboxes(UnionGeometry, UnionShapes, CollisionObjects, RootTriboxes, NextConvexId, bOptimizeConvexes);
	
	MainTribox.NodeTribox = FTribox(); 
	for(auto& RootTribox : RootTriboxes)
	{
		if(RootTribox.Value.NodeTribox.HasDatas())
		{
			MainTribox.NodeTribox += RootTribox.Value.NodeTribox;
		}
	}
	if(MainTribox.NodeTribox.HasDatas())
	{
		if(!MainTribox.TriboxConvex || (MainTribox.TriboxConvex && (MainTribox.TriboxConvex->AsA<FConvex>()->GetVolume() != MainTribox.NodeTribox.ComputeVolume())))
		{
			MainTribox.TriboxConvex = MainTribox.NodeTribox.MakeConvex();
			MainTribox.ConvexId = NextConvexId++;
		}

		// Build the tribox and add it to the list of optimized convexes
		SimplifiedConvexes.Add(MainTribox.TriboxConvex);
		Private::FImplicitBVH::CollectLeafObject(SimplifiedConvexes.Last(), FRigidTransform3::Identity, 0, CollisionObjects->ImplicitObjects, MainTribox.ConvexId);
	}
}

DECLARE_CYCLE_STAT(TEXT("Collisions::BuildUnionConnectivity"), STAT_BuildUnionConnectivity, STATGROUP_ChaosCollision);
void FConvexOptimizer::BuildUnionConnectivity(const Chaos::FImplicitObjectUnionPtr& UnionGeometry)
{
	SCOPE_CYCLE_COUNTER(STAT_BuildUnionConnectivity);
	
	const int32 NumTriboxes = UnionGeometry->GetObjects().Num();
	TUniquePtr<Private::FImplicitBVH> LocalBVH = nullptr;

	if(NumTriboxes > CVars::ChaosUnionBVHMinShapes)
	{
		TArray<Private::FImplicitBVHObject> ImplicitObjects;
		ImplicitObjects.Reserve(NumTriboxes);

		for (int32 RootIndex = 0; RootIndex < NumTriboxes; ++RootIndex)
		{
			if (auto* RootTribox = RootTriboxes.Find(UnionGeometry->GetObjects()[RootIndex]))
			{
				Private::FImplicitBVH::CollectLeafObject(UnionGeometry->GetObjects()[RootIndex], FRigidTransform3::Identity, RootIndex, ImplicitObjects, ImplicitObjects.Num());
			}
		}
		LocalBVH = FImplicitBVH::TryMakeFromLeaves(MoveTemp(ImplicitObjects), CVars::ChaosUnionBVHMinShapes, CVars::ChaosUnionBVHMaxDepth);
	}

	auto AddUnionEdges = [this, &UnionGeometry](const int32 RootIndexA, const int32 RootIndexB, const FAABB3& LocalBoundsA, FTriboxNode* RootTriboxA)
	{
		if (auto* RootTriboxB = RootTriboxes.Find(UnionGeometry->GetObjects()[RootIndexB]))
		{
			if(!RootTriboxB->bValidNode || (RootTriboxB->bValidNode && RootIndexA < RootIndexB))
			{ 
				if (LocalBoundsA.Intersects(RootTriboxB->NodeTribox.GetBounds()))
				{
					RootTriboxA->bValidEdges.Add(UnionGeometry->GetObjects()[RootIndexB], true);
					RootTriboxB->bValidEdges.Add(UnionGeometry->GetObjects()[RootIndexA], true);
				}
			}
		}
	};
	for (int32 RootIndexA = 0; RootIndexA < NumTriboxes; ++RootIndexA)
	{
		if (auto* RootTriboxA = RootTriboxes.Find(UnionGeometry->GetObjects()[RootIndexA]))
		{
			if (RootTriboxA->bValidNode)
			{
				const FAABB3 LocalBoundsA = RootTriboxA->NodeTribox.GetBounds();
				if (LocalBVH.IsValid())
				{
					LocalBVH->VisitAllIntersections(LocalBoundsA,
					[this, &RootIndexA, &LocalBoundsA, &UnionGeometry, &RootTriboxA, &AddUnionEdges](const FImplicitObject* ImplicitObjectB, const FRigidTransform3f& RelativeTransformB,
						const FAABB3f& RelativeBoundsB, const int32 RootIndexB, const int32 LeafObjectIndexB)
					{
						AddUnionEdges(RootIndexA, RootIndexB, LocalBoundsA, RootTriboxA);
					});
				}
				else
				{
					for (int32 RootIndexB = 0; RootIndexB < NumTriboxes; ++RootIndexB)
					{
						AddUnionEdges(RootIndexA, RootIndexB, LocalBoundsA, RootTriboxA);
					}
				}
			}
		}
	}
	for (auto& RootTribox : RootTriboxes)
	{
		RootTribox.Value.bValidNode = true;

		// TODO :  keep it for now since we must make it faster
		//RootTribox.Value.bValidEdges.KeySort([this, &UnionGeometry](const FImplicitObject& ImplicitObjectA, 
		//const FImplicitObject& ImplicitObjectB) -> bool{ return RootTriboxes.Find(&ImplicitObjectA)->ShapeIndex < RootTriboxes.Find(&ImplicitObjectB)->ShapeIndex;});
		for (auto& ValidEdge : RootTribox.Value.bValidEdges)
		{
			ValidEdge.Value = true;
		}
	}
}
	
DECLARE_CYCLE_STAT(TEXT("Collisions::MergeConnectedShapes"), STAT_MergeConnectedShapes, STATGROUP_ChaosCollision);
void FConvexOptimizer::MergeConnectedShapes(const Chaos::FImplicitObjectUnionPtr& UnionGeometry, TArray<FTriboxNode>& MergedNodes)
{
	SCOPE_CYCLE_COUNTER(STAT_MergeConnectedShapes);
	const int32 NumTriboxes = UnionGeometry->GetObjects().Num();
		
	MergedNodes.Reserve(RootTriboxes.Num());
	FTriboxNode* CurrentNode = nullptr;

	TArray<const FImplicitObject*> NodeQueue;
	NodeQueue.Reserve(NumTriboxes);

	for (int32 RootIndex = 0; RootIndex < NumTriboxes; ++RootIndex)
	{
		if (auto* RootTribox = RootTriboxes.Find(UnionGeometry->GetObjects()[RootIndex]))
		{
			if (RootTribox->bValidNode)
			{
				RootTribox->bValidNode = false;
				NodeQueue.Add(UnionGeometry->GetObjects()[RootIndex]);

				MergedNodes.Add({RootTribox->NodeTribox, nullptr, RootTribox->ShapeIndex, RootTribox->NodeVolume});
				CurrentNode = &MergedNodes.Last();

				while (!NodeQueue.IsEmpty())
				{
					const FImplicitObject* NextNode = NodeQueue.Pop(EAllowShrinking::No);

					for (auto& ValidEdge : RootTriboxes.Find(NextNode)->bValidEdges)
					{
						if (ValidEdge.Value)
						{
							ValidEdge.Value = false;
							const FImplicitObject* EdgeOtherNode = ValidEdge.Key;
							auto* OtherNode = RootTriboxes.Find(EdgeOtherNode);

							if (OtherNode->bValidNode)
							{
								FTribox LocalTribox = CurrentNode->NodeTribox;
								LocalTribox += OtherNode->NodeTribox;

								const FTribox::FRealType VolumeRatio = LocalTribox.ComputeVolume() / (CurrentNode->NodeVolume + OtherNode->NodeVolume);
								if (VolumeRatio < CVars::ChaosConvexSplittingThreshold)
								{
									CurrentNode->NodeTribox += OtherNode->NodeTribox;
									CurrentNode->NodeVolume += OtherNode->NodeVolume;

									NodeQueue.Add(EdgeOtherNode);
									OtherNode->bValidNode = false;
								}
							}
						}
					}
				}
			}
		}
	}
}

	
void FConvexOptimizer::BuildMultipleConvex(const Chaos::FImplicitObjectUnionPtr& UnionGeometry, const FShapesArray& UnionShapes, const bool bEnableMerging, const TBitArray<>& bOptimizeConvexes)
{
	BuildConvexTriboxes(UnionGeometry, UnionShapes, CollisionObjects, RootTriboxes, NextConvexId, bOptimizeConvexes);

	if(!bEnableMerging)
	{
		for(auto& RootTribox : RootTriboxes)
		{
			if(RootTribox.Value.NodeTribox.HasDatas())
			{
				// Build the Tribox and add it to the list of optimized convexes
				if(!RootTribox.Value.TriboxConvex || (RootTribox.Value.TriboxConvex && (RootTribox.Value.TriboxConvex->AsA<FConvex>()->GetVolume() != RootTribox.Value.NodeTribox.ComputeVolume())))
				{
					RootTribox.Value.TriboxConvex = RootTribox.Value.NodeTribox.MakeConvex();
					RootTribox.Value.ConvexId = NextConvexId++;
				}
				SimplifiedConvexes.Add(RootTribox.Value.TriboxConvex);
				Private::FImplicitBVH::CollectLeafObject(SimplifiedConvexes.Last(), FRigidTransform3::Identity, RootTribox.Value.ShapeIndex, CollisionObjects->ImplicitObjects, RootTribox.Value.ConvexId);
			}
		}
	}
	else
	{
		// Bottom-up merging to build the simplified convexes
		TArray<FConvexOptimizer::FTriboxNode> MergedNodes;

		// Create the connectivity graph
		BuildUnionConnectivity(UnionGeometry);

		// Merge the shapes
        MergeConnectedShapes(UnionGeometry, MergedNodes);

		// Add to the simplified convexes
        for(int32 NodeIndex = 0, NumMerged = MergedNodes.Num(); NodeIndex < NumMerged; ++NodeIndex)
        {
            if(MergedNodes[NodeIndex].NodeTribox.HasDatas() && (MergedNodes[NodeIndex].NodeVolume > CVars::ChaosConvexMinVolume))
            {
            	auto* RootTribox = RootTriboxes.Find(UnionGeometry->GetObjects()[MergedNodes[NodeIndex].ShapeIndex]);
            	if(!RootTribox->TriboxConvex || (RootTribox->TriboxConvex && (RootTribox->TriboxConvex->AsA<FConvex>()->GetVolume() != MergedNodes[NodeIndex].NodeTribox.ComputeVolume())))
            	{
            		RootTribox->TriboxConvex = MergedNodes[NodeIndex].NodeTribox.MakeConvex();
					RootTribox->ConvexId = NextConvexId++;
				}
            	SimplifiedConvexes.Add(RootTribox->TriboxConvex);
            	Private::FImplicitBVH::CollectLeafObject(SimplifiedConvexes.Last(), FRigidTransform3::Identity, MergedNodes[NodeIndex].ShapeIndex, CollisionObjects->ImplicitObjects, RootTribox->ConvexId);
            }
        }
	}
}

int32 FConvexOptimizer::NumCollisionObjects() const
{
	return BVH.IsValid() ? BVH->GetObjects().Num() : CollisionObjects->ImplicitObjects.Num();
}

void VisitCollisionObjects(const FConvexOptimizer* ConvexOptimizer, const FImplicitObject* ImplicitObject, const FImplicitHierarchyVisitor& VisitorFunc)
{
	if(ConvexOptimizer && ConvexOptimizer->IsValid())
	{
		ConvexOptimizer->VisitCollisionObjects(VisitorFunc);
	}
	else
	{
		ImplicitObject->VisitLeafObjects(VisitorFunc);
	}
}

void VisitOverlappingObjects(const FConvexOptimizer* ConvexOptimizer, const FImplicitObject* ImplicitObject, const FAABB3& LocalBounds, const FImplicitHierarchyVisitor& VisitorFunc)
{
	if(ConvexOptimizer && ConvexOptimizer->IsValid())
	{
		ConvexOptimizer->VisitOverlappingObjects(LocalBounds, VisitorFunc);
	}
	else
	{
		ImplicitObject->VisitOverlappingLeafObjects(LocalBounds, VisitorFunc);
	}
}

}
}
