// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/DebugDraw/DebugDrawImplicitObject.h"
#include "Chaos/Box.h"
#include "Chaos/Collision/ContactTriangles.h"
#include "Chaos/Capsule.h"
#include "Chaos/Convex.h"
#include "Chaos/DebugDraw/DebugDrawPrimitives.h"
#include "Chaos/HeightField.h"
#include "Chaos/ImplicitObject.h"
#include "Chaos/ImplicitObjectScaled.h"
#include "Chaos/ImplicitObjectTransformed.h"
#include "Chaos/Sphere.h"
#include "Chaos/TriangleMeshImplicitObject.h"
#include "ChaosDebugDraw/ChaosDDContext.h"
#include "ChaosDebugDraw/ChaosDDFrame.h"
#include "ChaosDebugDraw/ChaosDDRenderer.h"

#if CHAOS_DEBUG_DRAW

namespace Chaos
{
	void FChaosDDImplicitObject::Draw(
		const FConstImplicitObjectPtr& Implicit,
		const FRigidTransform3& Transform,
		const FColor& Color,
		float LineThickness, 
		float Duration)
	{
		ChaosDD::Private::FChaosDDContext::GetWriter().EnqueueCommand(
			[=](ChaosDD::Private::IChaosDDRenderer& Renderer)
			{
				Chaos::Private::ChaosDDRenderImplicitObject(Renderer, Implicit, Transform, Color, LineThickness, Duration);
			});
	}
}

namespace Chaos::Private
{
	void ChaosDDRenderSphere(
		ChaosDD::Private::IChaosDDRenderer& Renderer,
		const FImplicitSphere3& Sphere,
		const FRigidTransform3& Transform,
		const FColor& Color,
		float LineThickness,
		float Duration)
	{
		const FVec3 Scale = Transform.GetScale3D();
		Renderer.RenderSphere(Transform.TransformPosition(FVec3(Sphere.GetCenterf())), float(Scale.Z) * Sphere.GetRadiusf(), Color, LineThickness, Duration);
	}

	void ChaosDDRenderCapsule(
		ChaosDD::Private::IChaosDDRenderer& Renderer,
		const FImplicitCapsule3& Capsule,
		const FRigidTransform3& Transform,
		const FColor& Color,
		float LineThickness,
		float Duration)
	{
		const FVec3 Scale = Transform.GetScale3D();
		const FVec3 P = Transform.TransformPosition(FVec3(Capsule.GetCenterf()));
		const FRotation3 Q = Transform.GetRotation() * FRotationMatrix::MakeFromZ(Capsule.GetAxis());
		Renderer.RenderCapsule(P, Q, float(Scale.Z * (0.5 * Capsule.GetHeightf() + Capsule.GetRadiusf())), float(Scale.X * Capsule.GetRadiusf()), Color, LineThickness, Duration);
	}

	void ChaosDDRenderBox(
		ChaosDD::Private::IChaosDDRenderer& Renderer,
		const FImplicitBox3& Box,
		const FRigidTransform3& Transform,
		const FColor& Color,
		float LineThickness,
		float Duration)
	{
		const FVector3d Size = 0.5 * Transform.GetScale3D() * Box.Extents();
		const FVector3d Center = Transform.TransformPosition(Box.Center());

		Renderer.RenderBox(Center, Transform.GetRotation(), Size, Color, LineThickness, Duration);
	}

	void ChaosDDRenderConvex(
		ChaosDD::Private::IChaosDDRenderer& Renderer,
		const FImplicitConvex3& Convex,
		const FRigidTransform3& Transform,
		const FColor& Color,
		float LineThickness,
		float Duration)
	{
		for (int32 EdgeIndex = 0; EdgeIndex < Convex.NumEdges(); ++EdgeIndex)
		{
			const int32 EdgeVertexIndex0 = Convex.GetEdgeVertex(EdgeIndex, 0);
			const int32 EdgeVertexIndex1 = Convex.GetEdgeVertex(EdgeIndex, 1);
			const FVec3 EdgeVertex0 = Transform.TransformPosition(FVec3(Convex.GetVertex(EdgeVertexIndex0)));
			const FVec3 EdgeVertex1 = Transform.TransformPosition(FVec3(Convex.GetVertex(EdgeVertexIndex1)));

			Renderer.RenderLine(EdgeVertex0, EdgeVertex1, Color, LineThickness, Duration);
		}
	}

	template<typename MeshType>
	void ChaosDDRenderMesh(
		ChaosDD::Private::IChaosDDRenderer& Renderer,
		const MeshType& Mesh,
		const FRigidTransform3& Transform,
		const FColor& Color,
		float LineThickness,
		float Duration)
	{
		const FSphere3d RegionOfInterest = Renderer.GetDrawRegion();
		const FAABB3 WorldQueryBounds = FAABB3(RegionOfInterest.Center - FVec3(RegionOfInterest.W), RegionOfInterest.Center + FVec3(RegionOfInterest.W));
		const FAABB3 LocalQueryBounds = WorldQueryBounds.InverseTransformedAABB(Transform);

		Mesh.VisitTriangles(LocalQueryBounds, Transform, 
			[&Renderer, &Color, LineThickness, Duration, &RegionOfInterest](const FTriangle& Tri, const int32 TriangleIndex, const int32 VertexIndex0, const int32 VertexIndex1, const int32 VertexIndex2)
			{
				const FReal RegionDistanceSq = (Tri.GetCentroid() - FVec3(RegionOfInterest.Center)).SizeSquared();
				if (RegionDistanceSq < FMath::Square(RegionOfInterest.W))
				{
					// @todo(chaos): could keep track of edges drawn so we don't add every shared edge twice...
					Renderer.RenderTriangle(Tri[0], Tri[1], Tri[2], Color, LineThickness, Duration);
				}
			});
	}

	void ChaosDDRenderLeafImplicitObject(
		ChaosDD::Private::IChaosDDRenderer& Renderer,
		const FImplicitObject* Implicit,
		const FRigidTransform3& Transform,
		const FColor& Color,
		float LineThickness,
		float Duration)
	{
		if (const FImplicitSphere3* Sphere = Implicit->AsA<FImplicitSphere3>())
		{
			ChaosDDRenderSphere(Renderer, *Sphere, Transform, Color, LineThickness, Duration);
		}
		else if (const FImplicitCapsule3* Capsule = Implicit->AsA<FImplicitCapsule3>())
		{
			ChaosDDRenderCapsule(Renderer, *Capsule, Transform, Color, LineThickness, Duration);
		}
		else if (const FImplicitBox3* Box = Implicit->AsA<FImplicitBox3>())
		{
			ChaosDDRenderBox(Renderer, *Box, Transform, Color, LineThickness, Duration);
		}
		else if (const FImplicitConvex3* Convex = Implicit->AsA<FImplicitConvex3>())
		{
			ChaosDDRenderConvex(Renderer, *Convex, Transform, Color, LineThickness, Duration);
		}
		else if (const FTriangleMeshImplicitObject* TriMesh = Implicit->AsA<FTriangleMeshImplicitObject>())
		{
			ChaosDDRenderMesh(Renderer, *TriMesh, Transform, Color, LineThickness, Duration);
		}
		else if (const FHeightField* HeightField = Implicit->AsA<FHeightField>())
		{
			ChaosDDRenderMesh(Renderer, *HeightField, Transform, Color, LineThickness, Duration);
		}
		else if (const FImplicitObjectInstanced* Instanced = Implicit->AsA<FImplicitObjectInstanced>())
		{
			ChaosDDRenderLeafImplicitObject(Renderer, Instanced->GetInnerObject().Get(), Transform, Color, LineThickness, Duration);
		}
		else if (const FImplicitObjectScaled* Scaled = Implicit->AsA<FImplicitObjectScaled>())
		{
			const FRigidTransform3 ScaledTransform = FRigidTransform3(Transform.GetTranslation(), Transform.GetRotation(), Transform.GetScale3D() * Scaled->GetScale());
			ChaosDDRenderLeafImplicitObject(Renderer, Scaled->GetInnerObject().Get(), ScaledTransform, Color, LineThickness, Duration);
		}
	}

	void ChaosDDRenderImplicitObject(
		ChaosDD::Private::IChaosDDRenderer& Renderer,
		const FConstImplicitObjectPtr& RootImplicit,
		const FRigidTransform3& RootTransform,
		const FColor& Color,
		float LineThickness,
		float Duration)
	{
		RootImplicit->VisitLeafObjects(
			[&](const FImplicitObject* LeafImplicitObject, const FRigidTransform3& LeafRelativeTransform, const int32 UnusedRootObjectIndex, const int32 UnusedObjectIndex, const int32 UnusedLeafObjectIndex)
			{
				const FRigidTransform3 LeafTransform = LeafRelativeTransform * RootTransform;
				ChaosDDRenderLeafImplicitObject(Renderer, LeafImplicitObject, LeafTransform, Color, LineThickness, Duration);
			});
	}
}

#endif