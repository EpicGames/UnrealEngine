// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/DebugDraw/DebugDrawParticle.h"
#include "Chaos/ConvexOptimizer.h"
#include "Chaos/DebugDraw/DebugDrawImplicitObject.h"
#include "Chaos/GeometryParticles.h"
#include "Chaos/Particle/ObjectState.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/ParticleIterator.h"
#include "Chaos/PBDRigidParticles.h"
#include "Chaos/ShapeInstance.h"
#include "Chaos/Island/IslandManager.h"
#include "ChaosDebugDraw/ChaosDDContext.h"
#include "ChaosDebugDraw/ChaosDDFrame.h"
#include "ChaosDebugDraw/ChaosDDRenderer.h"

// For FChaosDebugDrawSettings - move that class when the old debug draw is no longer used
#include "Chaos/ChaosDebugDraw.h"

#if CHAOS_DEBUG_DRAW

namespace Chaos::DebugDraw
{
	extern bool bChaosDebugDebugDrawColorShapesByShapeType;
	extern bool bChaosDebugDebugDrawColorShapesByIsland;
	extern bool bChaosDebugDebugDrawColorShapesByInternalCluster;
	extern bool bChaosDebugDebugDrawColorShapesBySimQueryType;
	extern bool bChaosDebugDebugDrawColorShapesByConvexType;
	extern bool bChaosDebugDebugDrawColorShapesByClusterUnion;
	
	extern bool bChaosDebugDebugDrawColorBoundsByShapeType;

	extern bool bChaosDebugDebugDrawShowQueryOnlyShapes;
	extern bool bChaosDebugDebugDrawShowSimOnlyShapes;
	extern bool bChaosDebugDebugDrawShowProbeOnlyShapes;

	extern bool bChaosDebugDebugDrawShowOptimizedConvexes;
}

namespace Chaos::CVars
{
	extern int32 ChaosSolverDrawShapesShowStatic;
	extern int32 ChaosSolverDrawShapesShowKinematic;
	extern int32 ChaosSolverDrawShapesShowDynamic;
	extern int32 ChaosSolverDebugDrawShowServer;
	extern int32 ChaosSolverDebugDrawShowClient;
	extern int32 ChaosSolverDebugDrawColorShapeByClientServer;

	extern Chaos::DebugDraw::FChaosDebugDrawSettings ChaosSolverDebugDebugDrawSettings;

	extern DebugDraw::FChaosDebugDrawColorsByState GetSolverShapesColorsByState_Server();
	extern DebugDraw::FChaosDebugDrawColorsByState GetSolverShapesColorsByState_Client();
}

namespace Chaos
{
	FColor GetIndexColor(const int32 Index)
	{
		static const FColor Colors[] =
		{
			FColor::Red,
			FColor::Orange,
			FColor::Yellow,
			FColor::Green,
			FColor::Emerald,
			FColor::Cyan,
			FColor::Turquoise,
			FColor::Blue,
			FColor::Magenta,
			FColor::Purple,
		};
		const int32 NumColors = UE_ARRAY_COUNT(Colors);

		return Colors[Index % NumColors];
	};


	FColor GetIslandColor(const int32 IslandIndex, const bool bIsAwake)
	{
		static const FColor SleepingColor = FColor::Black;
		static const FColor NullColor = FColor::White;

		if (IslandIndex == INDEX_NONE)
		{
			return NullColor;
		}

		if (!bIsAwake)
		{
			return SleepingColor;
		}

		return GetIndexColor(IslandIndex);
	};

	struct FChaosDDParticleData
	{
		int32 IslandId;
		int32 ClusterId;
		EObjectStateType ObjectState;
		uint8 bIsClusterUnion : 1;
		uint8 bIsInternalCluster : 1;
		uint8 bIsOneWay : 1;

		FChaosDDParticleData(
			const FConstGenericParticleHandle& InParticle)
		{
			check(InParticle.IsValid());
			IslandId = (InParticle->GetConstraintGraphNode() != nullptr) ? InParticle->GetConstraintGraphNode()->GetIslandId() : INDEX_NONE;
			ClusterId = (InParticle->CastToClustered() != nullptr) ? FMath::Abs(InParticle->CastToClustered()->ClusterGroupIndex()) : INDEX_NONE;
			ObjectState = InParticle->ObjectState();
			bIsClusterUnion = (InParticle->CastToClustered() != nullptr) ? (InParticle->CastToClustered()->PhysicsProxy()->GetType() == EPhysicsProxyType::ClusterUnionProxy) : false;
			bIsInternalCluster = (InParticle->CastToClustered() != nullptr) ? InParticle->CastToClustered()->InternalCluster() : false;
			bIsOneWay = InParticle->OneWayInteraction();
		}
	};

	struct FChaosDDShapeData
	{
		EChaosCollisionTraceFlag CollisionTraceFlag;
		uint8 bIsQuery: 1;
		uint8 bIsSim: 1;
		uint8 bIsProbe: 1;
		uint8 bIsNonOptimized : 1;
		uint8 bIsOptimized : 1;

		FChaosDDShapeData(
			const FShapeInstance* InShapeInstance,
			bool bInIsNonOptimized,
			bool bInIsOptimized)
		{
			check(InShapeInstance != nullptr);
			CollisionTraceFlag = InShapeInstance->GetCollisionTraceType();
			bIsQuery = InShapeInstance->GetQueryEnabled();
			bIsSim = InShapeInstance->GetSimEnabled();
			bIsProbe = InShapeInstance->GetIsProbe();
			bIsNonOptimized = bInIsNonOptimized;
			bIsOptimized = bInIsOptimized;
		}
	};

	// Helper for collecting the data used to render particle shapes
	struct FChaosDDParticleShape
	{
	public:
		// Determine whether we should render based on the flags
		static bool ShouldRender(const FConstImplicitObjectPtr& Implicit, FChaosDDParticleData ParticleData, FChaosDDShapeData ShapeData, bool bIsServer)
		{
			const Chaos::DebugDraw::FChaosDebugDrawSettings& Settings = Chaos::CVars::ChaosSolverDebugDebugDrawSettings;

			if (!Chaos::CVars::ChaosSolverDrawShapesShowStatic && ParticleData.ObjectState == EObjectStateType::Static)
			{
				return false;
			}
			if (!Chaos::CVars::ChaosSolverDrawShapesShowKinematic && ParticleData.ObjectState == EObjectStateType::Kinematic)
			{
				return false;
			}
			if (!Chaos::CVars::ChaosSolverDrawShapesShowDynamic && ParticleData.ObjectState == EObjectStateType::Dynamic)
			{
				return false;
			}

			if (!Chaos::DebugDraw::bChaosDebugDebugDrawShowQueryOnlyShapes && ShapeData.bIsQuery && !ShapeData.bIsSim && !ShapeData.bIsProbe)
			{
				return false;
			}
			if (!Chaos::DebugDraw::bChaosDebugDebugDrawShowSimOnlyShapes && !ShapeData.bIsQuery && ShapeData.bIsSim && !ShapeData.bIsProbe)
			{
				return false;
			}
			if (!Chaos::DebugDraw::bChaosDebugDebugDrawShowProbeOnlyShapes && !ShapeData.bIsQuery && !ShapeData.bIsSim && ShapeData.bIsProbe)
			{
				return false;
			}
			if (!Chaos::CVars::ChaosSolverDebugDrawShowServer && bIsServer)
			{
				return false;
			}
			if (!Chaos::CVars::ChaosSolverDebugDrawShowClient && !bIsServer)
			{
				return false;
			}

			if (!ShapeData.bIsNonOptimized && !DebugDraw::bChaosDebugDebugDrawShowOptimizedConvexes)
			{
				return false;
			}

			if (!ShapeData.bIsOptimized && DebugDraw::bChaosDebugDebugDrawShowOptimizedConvexes)
			{
				return false;
			}

			// Depending on the shape settings, we may not show the simple or complex shape
			const EImplicitObjectType InnerType = GetInnerType(Implicit->GetType());
			const bool bIsMesh = (InnerType == ImplicitObjectType::TriangleMesh);
			const bool bShowMeshes = (Settings.bShowComplexCollision && (ShapeData.CollisionTraceFlag != EChaosCollisionTraceFlag::Chaos_CTF_UseSimpleAsComplex))
				|| (Settings.bShowSimpleCollision && (ShapeData.CollisionTraceFlag == EChaosCollisionTraceFlag::Chaos_CTF_UseComplexAsSimple));
			const bool bShowNonMeshes = (Settings.bShowSimpleCollision && (ShapeData.CollisionTraceFlag != EChaosCollisionTraceFlag::Chaos_CTF_UseComplexAsSimple))
				|| (Settings.bShowComplexCollision && (ShapeData.CollisionTraceFlag == EChaosCollisionTraceFlag::Chaos_CTF_UseSimpleAsComplex));
			if (bIsMesh && !bShowMeshes)
			{
				return false;
			}
			else if (!bIsMesh && !bShowNonMeshes)
			{
				return false;
			}

			return true;
		}

		// Determine the color from the flags
		static FColor GetRenderColor(const FConstImplicitObjectPtr& Implicit, FChaosDDParticleData ParticleData, FChaosDDShapeData ShapeData, bool bIsServer)
		{
			const Chaos::DebugDraw::FChaosDebugDrawSettings& Settings = Chaos::CVars::ChaosSolverDebugDebugDrawSettings;

			const EImplicitObjectType InnerType = GetInnerType(Implicit->GetType());
			if (Chaos::CVars::ChaosSolverDebugDrawColorShapeByClientServer)
			{
				if (bIsServer)
				{
					return Chaos::CVars::GetSolverShapesColorsByState_Server().GetColorFromState(ParticleData.ObjectState, ParticleData.bIsOneWay);
				}
				else
				{
					return Chaos::CVars::GetSolverShapesColorsByState_Client().GetColorFromState(ParticleData.ObjectState, ParticleData.bIsOneWay);
				}
			}
			if (Chaos::DebugDraw::bChaosDebugDebugDrawColorShapesByShapeType)
			{
				return Settings.ShapesColorsPerShapeType.GetColorFromShapeType(InnerType);
			}
			if (Chaos::DebugDraw::bChaosDebugDebugDrawColorShapesByIsland)
			{
				return GetIslandColor(ParticleData.IslandId, true);
			}
			if (Chaos::DebugDraw::bChaosDebugDebugDrawColorShapesByInternalCluster)
			{
				if (ParticleData.bIsClusterUnion)
				{
					if (Chaos::DebugDraw::bChaosDebugDebugDrawColorShapesByClusterUnion)
					{
						return GetIndexColor(ParticleData.ClusterId);
					}
					else
					{
						if (ParticleData.bIsInternalCluster)
						{
							return FColor::Purple;
						}
					}
				}
				return FColor::Black;
			}
			if (Chaos::DebugDraw::bChaosDebugDebugDrawColorShapesByConvexType)
			{
				if (InnerType == ImplicitObjectType::Convex)
				{
					if (!ShapeData.bIsNonOptimized)
					{
						return FColor::Green;
					}
					return FColor::Orange;
				}
			}

			return Settings.ShapesColorsPerState.GetColorFromState(ParticleData.ObjectState, ParticleData.bIsOneWay);
		}

		static float GetLineThickness()
		{
			const Chaos::DebugDraw::FChaosDebugDrawSettings& Settings = Chaos::CVars::ChaosSolverDebugDebugDrawSettings;

			return Settings.LineThickness;
		}

		static float GetDuration()
		{
			return 0.0f;
		}

		static int32 GetParticleCommandCost(const FConstGenericParticleHandle& InParticle)
		{
			return 1;
		}

		static void Draw(
			const FRigidTransform3& SpaceTransform,
			const FConstGenericParticleHandle& InParticle,
			const FShapeInstance* InShapeInstance,
			const FImplicitObject* InImplicitObject,
			const FRigidTransform3& InRelativeTransform,
			bool bInIsNonOptimized,
			bool bInIsOptimized,
			bool bInAutoColor,
			const FColor& InColor = FColor::Purple)
		{
			if (!InParticle.IsValid())
			{
				return;
			}
			if (InShapeInstance == nullptr)
			{
				return;
			}

			const int32 Cost = GetParticleCommandCost(InParticle);
			const FBox3d Bounds = FBox3d((InParticle->WorldSpaceInflatedBounds().Min()), InParticle->WorldSpaceInflatedBounds().Max()).TransformBy(SpaceTransform);

			ChaosDD::Private::FChaosDDFrameWriter Writer = ChaosDD::Private::FChaosDDContext::GetWriter();

			if (!Writer.IsInDrawRegion(Bounds))
			{
				return;
			}

			if (!Writer.AddToCost(Cost))
			{
				return;
			}

			const FRigidTransform3 ParticleTransform = InParticle->GetTransformPQ() * SpaceTransform;
			const FChaosDDParticleData ParticleData = FChaosDDParticleData(InParticle);
			const FChaosDDShapeData ShapeData = FChaosDDShapeData(InShapeInstance, bInIsNonOptimized, bInIsOptimized);

			Writer.EnqueueCommand(
				[
					Transform = InRelativeTransform * ParticleTransform,
					ImplicitObject = FConstImplicitObjectPtr(InImplicitObject),
					ParticleData,
					ShapeData,
					InColor,
					bInAutoColor
				]
				(ChaosDD::Private::IChaosDDRenderer& Renderer)
				{
					if (ShouldRender(ImplicitObject.GetReference(), ParticleData, ShapeData, Renderer.IsServer()))
					{
						Chaos::Private::ChaosDDRenderImplicitObject(
							Renderer,
							ImplicitObject.GetReference(),
							Transform,
							bInAutoColor ? GetRenderColor(ImplicitObject.GetReference(), ParticleData, ShapeData, Renderer.IsServer()) : InColor,
							GetLineThickness(),
							GetDuration());
					}
				});
		}

		static void Draw(
			const FRigidTransform3& SpaceTransform, 
			const FConstGenericParticleHandle& InParticle,
			const FShapeInstance* InShapeInstance,
			bool bInIsNonOptimized,
			bool bInIsOptimized,
			bool bInAutoColor,
			const FColor& InColor = FColor::Purple)
		{
			if (!InParticle.IsValid())
			{
				return;
			}
			if (InShapeInstance == nullptr)
			{
				return;
			}

			const FBox3d Bounds = FBox3d((InParticle->WorldSpaceInflatedBounds().Min()), InParticle->WorldSpaceInflatedBounds().Max()).TransformBy(SpaceTransform);

			ChaosDD::Private::FChaosDDFrameWriter Writer = ChaosDD::Private::FChaosDDContext::GetWriter();

			if (!Writer.IsInDrawRegion(Bounds))
			{
				return;
			}

			const FChaosDDParticleData ParticleData = FChaosDDParticleData(InParticle);
			const FChaosDDShapeData ShapeData = FChaosDDShapeData(InShapeInstance, bInIsNonOptimized, bInIsOptimized);

			InShapeInstance->GetGeometry()->VisitLeafObjects(
				[&Writer, &SpaceTransform, InParticle, InShapeInstance, bInIsNonOptimized, bInIsOptimized, bInAutoColor, &InColor](const FImplicitObject* LeafImplicitObject, const FRigidTransform3& LeafRelativeTransform, const int32 UnusedRootObjectIndex, const int32 UnusedObjectIndex, const int32 UnusedLeafObjectIndex)
				{
					FChaosDDParticleShape::Draw(SpaceTransform, InParticle, InShapeInstance, LeafImplicitObject, LeafRelativeTransform, bInIsNonOptimized, bInIsOptimized, bInAutoColor, InColor);
				});

		}

	};

	void FChaosDDParticle::DrawShapes(
		const FRigidTransform3& SpaceTransform,
		const FGeometryParticleHandle* InParticleHandle)
	{
		// Record the optimized geometry if there is any.
		// If there is no optimized geo, we report the regular geometry as the optimized geometry as well 
		// as the non-optimized so that DebugDrawOptimized will show regular geo instead.
		bool bHasOptimizedShapes = DrawOptimizedShapes(InParticleHandle);

		for (const Chaos::FShapeInstancePtr& ShapeInstance : InParticleHandle->ShapeInstances())
		{
			FChaosDDParticleShape::Draw(SpaceTransform, InParticleHandle, ShapeInstance.Get(), /*bIsNonOptimized*/true, /*bIsOptimized*/!bHasOptimizedShapes, /*bAutoColor*/true);
		}
	}

	void FChaosDDParticle::DrawShapes(
		const FGeometryParticleHandle* InParticleHandle)
	{
		FChaosDDParticle::DrawShapes(FRigidTransform3::Identity, InParticleHandle);
	}

	void FChaosDDParticle::DrawShapes(
		const FGeometryParticleHandle* InParticleHandle,
		const FColor& Color)
	{
		for (const Chaos::FShapeInstancePtr& ShapeInstance : InParticleHandle->ShapeInstances())
		{
			FChaosDDParticleShape::Draw(FRigidTransform3::Identity, InParticleHandle, ShapeInstance.Get(), /*bIsNonOptimized*/true, /*bIsOptimized*/false, /*bAutoColor*/false, Color);
		}
	}

	bool FChaosDDParticle::DrawOptimizedShapes(
		const FGeometryParticleHandle* InParticleHandle)
	{
		const Private::FConvexOptimizer* ConvexOptimizer = InParticleHandle->CastToClustered() ? InParticleHandle->CastToClustered()->ConvexOptimizer().Get() : nullptr;
		if ((ConvexOptimizer == nullptr) || !ConvexOptimizer->IsValid())
		{
			return false;
		}

		const auto& DrawOptimizedConvex = 
			[InParticleHandle, ConvexOptimizer](const FImplicitObject* ImplicitObject, const FRigidTransform3& RelativeTransform, const int32 RootObjectIndex, const int32 ObjectIndex, const int32 LeafObjectIndex)
			{
				// The convex optimizer may reuse ShapeInstances from the particle, but with a different implicit object
				// It's like it is pretending to be a Union-of-Unions (in which case the shapes in the child unions share 
				// a ShapeInstance)
				const FShapeInstance* ShapeInstance = nullptr;
				if ((RootObjectIndex == INDEX_NONE) && (ConvexOptimizer->GetShapeInstances().Num() > 0))
				{
					ShapeInstance = ConvexOptimizer->GetShapeInstances()[0].Get();
				}
				else
				{
					const int32 ShapeIndex = (InParticleHandle->ShapeInstances().IsValidIndex(RootObjectIndex)) ? RootObjectIndex : 0;
					ShapeInstance = InParticleHandle->ShapeInstances()[ShapeIndex].Get();
				}

				FChaosDDParticleShape::Draw(FRigidTransform3::Identity, InParticleHandle, ShapeInstance, ImplicitObject, RelativeTransform, /*bIsNonOptimized*/false, /*bIsOptimized*/true, /*bAutoColor*/true);
			};

		ConvexOptimizer->VisitCollisionObjects(DrawOptimizedConvex);

		return true;
	}

}

#endif