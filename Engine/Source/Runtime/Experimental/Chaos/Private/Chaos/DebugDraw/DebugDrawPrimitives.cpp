// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/DebugDraw/DebugDrawPrimitives.h"
#include "ChaosDebugDraw/ChaosDDContext.h"
#include "ChaosDebugDraw/ChaosDDFrame.h"
#include "ChaosDebugDraw/ChaosDDRenderer.h"

#if CHAOS_DEBUG_DRAW

namespace Chaos
{
	void FChaosDDPrimitives::DrawPoint(const FVector3d& Position, const FColor& Color, float PointSize, float Duration)
	{
		constexpr int32 Cost = 1;
		const FBox3d Bounds = FBox3d(Position, Position);

		ChaosDD::Private::FChaosDDContext::GetWriter().TryEnqueueCommand(Cost, Bounds,
			[=](ChaosDD::Private::IChaosDDRenderer& Renderer)
			{
				Renderer.RenderPoint(Position, Color, PointSize, Duration);
			});
	}

	void FChaosDDPrimitives::DrawLine(const FVector3d& A, const FVector3d& B, const FColor& Color, float LineThickness, float Duration)
	{
		constexpr int32 Cost = 1;
		const FBox3d Bounds = FBox3d(FVector3d::Min(A, B), FVector3d::Max(A, B));

		ChaosDD::Private::FChaosDDContext::GetWriter().TryEnqueueCommand(Cost, Bounds,
			[=](ChaosDD::Private::IChaosDDRenderer& Renderer)
			{
				Renderer.RenderLine(A, B, Color, LineThickness, Duration);
			});
	}

	void FChaosDDPrimitives::DrawArrow(const FVector3d& A, const FVector3d& B, float ArrowSize, const FColor& Color, float LineThickness, float Duration)
	{
		constexpr int32 Cost = 1;
		const FBox3d Bounds = FBox3d(FVector3d::Min(A, B), FVector3d::Max(A, B));

		ChaosDD::Private::FChaosDDContext::GetWriter().TryEnqueueCommand(Cost, Bounds,
			[=](ChaosDD::Private::IChaosDDRenderer& Renderer)
			{
				Renderer.RenderArrow(A, B, ArrowSize, Color, LineThickness, Duration);
			});
	}
	
	void FChaosDDPrimitives::DrawCircle(const FVector3d& Center, const FMatrix& Axes, float Radius, const FColor& Color, float LineThickness, float Duration)
	{
		constexpr int32 Cost = 1;
		const FBox3d Bounds = FBox3d(Center - FVector3d(Radius), Center + FVector3d(Radius));

		ChaosDD::Private::FChaosDDContext::GetWriter().TryEnqueueCommand(Cost, Bounds,
			[=](ChaosDD::Private::IChaosDDRenderer& Renderer)
			{
				Renderer.RenderCircle(Center, Axes, Radius, Color, LineThickness, Duration);
			});
	}

	void FChaosDDPrimitives::DrawSphere(const FVector3d& Center, const float Radius, const FColor& Color, float LineThickness, float Duration)
	{
		constexpr int32 Cost = 64;
		const FBox3d Bounds = FBox3d(Center - FVector3d(Radius), Center + FVector3d(Radius));

		ChaosDD::Private::FChaosDDContext::GetWriter().TryEnqueueCommand(Cost, Bounds,
			[=](ChaosDD::Private::IChaosDDRenderer& Renderer)
			{
				Renderer.RenderSphere(Center, Radius, Color, LineThickness, Duration);
			});
	}

	void FChaosDDPrimitives::DrawCapsule(const FVector3d& Center, const FQuat4d& Rotation, float HalfHeight, float Radius, const FColor& Color, float LineThickness, float Duration)
	{
		constexpr int32 Cost = 16;
		const FVector3d EndOffset = HalfHeight * (Rotation * FVector3d::UnitZ());
		const FVector3d A = Center - EndOffset;
		const FVector3d B = Center + EndOffset;
		const FBox3d Bounds = FBox3d(FVector3d::Min(A, B), FVector3d::Max(A, B));

		ChaosDD::Private::FChaosDDContext::GetWriter().TryEnqueueCommand(Cost, Bounds,
			[=](ChaosDD::Private::IChaosDDRenderer& Renderer)
			{
				Renderer.RenderCapsule(Center, Rotation, HalfHeight, Radius, Color, LineThickness, Duration);
			});
	}

	void FChaosDDPrimitives::DrawBox(const FVector3d& Center, const FQuat4d& Rotation, const FVector3d& Size, const FColor& Color, float LineThickness, float Duration)
	{
		constexpr int32 Cost = 12;
		const FBox3d Bounds = FBox3d(-0.5 * Size, 0.5 * Size).TransformBy(FTransform(Rotation, Center));

		ChaosDD::Private::FChaosDDContext::GetWriter().TryEnqueueCommand(Cost, Bounds,
			[=](ChaosDD::Private::IChaosDDRenderer& Renderer)
			{
				Renderer.RenderBox(Center, Rotation, Size, Color, LineThickness, Duration);
			});
	}

	void FChaosDDPrimitives::DrawTriangle(const FVector3d& A, const FVector3d& B, const FVector3d& C, const FColor& Color, float LineThickness, float Duration)
	{
		constexpr int32 Cost = 3;
		const FBox3d Bounds = FBox3d(FVector3d::Min3(A, B, C), FVector3d::Max3(A, B, C));

		ChaosDD::Private::FChaosDDContext::GetWriter().TryEnqueueCommand(Cost, Bounds,
			[=](ChaosDD::Private::IChaosDDRenderer& Renderer)
			{
				Renderer.RenderTriangle(A, B, C, Color, LineThickness, Duration);
			});
	}

	void FChaosDDPrimitives::DrawString(const FVector3d& TextLocation, const FString& Text, const FColor& Color, float FontScale, bool bDrawShadow, float Duration)
	{
		const int32 Cost = 10;
		const FBox3d Bounds = FBox3d(TextLocation, TextLocation);

		ChaosDD::Private::FChaosDDContext::GetWriter().TryEnqueueCommand(Cost, Bounds,
			[=](ChaosDD::Private::IChaosDDRenderer& Renderer)
			{
				Renderer.RenderString(TextLocation, Text, Color, FontScale, bDrawShadow, Duration);
			});
	}
}

#endif