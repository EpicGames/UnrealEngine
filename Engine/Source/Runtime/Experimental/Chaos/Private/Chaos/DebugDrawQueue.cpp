// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/DebugDrawQueue.h"
#include "HAL/IConsoleManager.h"
#include "Misc/ScopeLock.h"

#if CHAOS_DEBUG_DRAW

namespace Chaos
{
	bool bChaosDebugDraw_UseNewQueue = true;
	FAutoConsoleVariableRef CVarChaos_DebugDraw_UseNewQueue(TEXT("p.Chaos.DebugDraw.UseNewQueue"), bChaosDebugDraw_UseNewQueue, TEXT(""));

	bool bChaosDebugDraw_UseLegacyQueue = false;
	FAutoConsoleVariableRef CVarChaos_DebugDraw_UseLegacyQueue(TEXT("p.Chaos.DebugDraw.UseLegacyQueue"), bChaosDebugDraw_UseLegacyQueue, TEXT(""));

	void FDebugDrawQueue::SetConsumerActive(void* Consumer, bool bConsumerActive)
	{
		FScopeLock Lock(&ConsumersCS);
	
		if(bConsumerActive)
		{
			Consumers.AddUnique(Consumer);
		}
		else
		{
			Consumers.Remove(Consumer);
		}
	
		NumConsumers = Consumers.Num();
	}
	
	FDebugDrawQueue& FDebugDrawQueue::GetInstance()
	{
		static FDebugDrawQueue* PSingleton = nullptr;
		if(PSingleton == nullptr)
		{
			static FDebugDrawQueue Singleton;
			PSingleton = &Singleton;
		}
		return *PSingleton;
	}
}

#endif
