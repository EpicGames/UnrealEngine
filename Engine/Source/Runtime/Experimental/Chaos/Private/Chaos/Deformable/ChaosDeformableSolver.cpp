// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/Deformable/ChaosDeformableSolver.h"
#include "Chaos/Deformable/ChaosDeformableSolverTypes.h"
#include "Chaos/Deformable/ChaosDeformableSolverProxy.h"
#include "Chaos/Deformable/ChaosDeformableConstraintsProxy.h"
#include "Chaos/Deformable/ChaosDeformableCollisionsProxy.h"

#include "ChaosLog.h"
#include "Chaos/BoundingVolumeHierarchy.h"
#include "Chaos/DebugDrawQueue.h"
#include "Chaos/Tetrahedron.h"
#include "Chaos/TriangleMesh.h"
#include "Chaos/PBDAltitudeSpringConstraints.h"
#include "Chaos/PBDBendingConstraints.h"
#include "Chaos/PBDSpringConstraints.h"
#include "Chaos/PBDVolumeConstraint.h"
#include "Chaos/PBDSpringConstraints.h"
#include "Chaos/PBDTriangleMeshCollisions.h"
#include "Chaos/PBDCollisionSpringConstraints.h"
#include "Chaos/PBDParticles.h"
#include "Chaos/PBDSoftsSolverParticles.h"
#include "Chaos/PBDTetConstraints.h"
#include "Chaos/PerParticleGravity.h"
#include "Chaos/XPBDCorotatedConstraints.h"
#include "Chaos/XPBDVolumeConstraints.h"
#include "Chaos/XPBDCorotatedFiberConstraints.h"
#include "Chaos/Plane.h"
#include "Chaos/Utilities.h"
#include "Chaos/PBDEvolution.h"
#include "Containers/ContainersFwd.h"
#include "Containers/StringConv.h"
#include "Containers/Set.h"
#include "CoreMinimal.h"
#include "GeometryCollection/Facades/CollectionKinematicBindingFacade.h"
#include "GeometryCollection/Facades/CollectionVertexBoneWeightsFacade.h"
#include "GeometryCollection/Facades/CollectionPositionTargetFacade.h"
#include "GeometryCollection/Facades/CollectionCollisionFacade.h"
#include "GeometryCollection/Facades/CollectionConstraintOverrideFacade.h"
#include "GeometryCollection/Facades/CollectionMeshFacade.h"
#include "GeometryCollection/Facades/CollectionMuscleActivationFacade.h"
#include "GeometryCollection/Facades/CollectionTetrahedralFacade.h"
#include "GeometryCollection/Facades/CollectionVolumeConstraintFacade.h"
#include "Misc/FileHelper.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/IConsoleManager.h"
#include "Misc/Paths.h"

#define PERF_SCOPE(X) SCOPE_CYCLE_COUNTER(X); TRACE_CPUPROFILER_EVENT_SCOPE(X);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.Solver.Constructor"), STAT_ChaosDeformableSolver_Constructor, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.Solver.Destructor"), STAT_ChaosDeformableSolver_Destructor, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.Solver.UpdateProxyInputPackages"), STAT_ChaosDeformableSolver_UpdateProxyInputPackages, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.Solver.Simulate"), STAT_ChaosDeformableSolver_Simulate, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.Solver.AdvanceDt."), STAT_ChaosDeformableSolver_AdvanceDt, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.Solver.Reset"), STAT_ChaosDeformableSolver_Reset, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.Solver.Update"), STAT_ChaosDeformableSolver_Update, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.Solver.UpdateOutputState"), STAT_ChaosDeformableSolver_UpdateOutputState, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.Solver.PullOutputPackage"), STAT_ChaosDeformableSolver_PullOutputPackage, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.Solver.PushOutputPackage"), STAT_ChaosDeformableSolver_PushOutputPackage, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.Solver.PullInputPackage"), STAT_ChaosDeformableSolver_PullInputPackage, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.Solver.PushInputPackage"), STAT_ChaosDeformableSolver_PushInputPackage, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.Solver.PullRestartPackage"), STAT_ChaosDeformableSolver_PullRestartPackage, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.Solver.PushRestartPackage"), STAT_ChaosDeformableSolver_PushRestartPackage, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.Solver.InitializeSimulationObjects"), STAT_ChaosDeformableSolver_InitializeSimulationObjects, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.Solver.InitializeSimulationObject"), STAT_ChaosDeformableSolver_InitializeSimulationObject, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.Solver.InitializeDeformableParticles"), STAT_ChaosDeformableSolver_InitializeDeformableParticles, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.Solver.InitializeKinematicParticles"), STAT_ChaosDeformableSolver_InitializeKinematicParticles, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.Solver.InitializeTetrahedralOrTriangleConstraint"), STAT_ChaosDeformableSolver_InitializeTetrahedralOrTriangleConstraint, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.Solver.InitializeGridBasedConstraints"), STAT_ChaosDeformableSolver_InitializeGridBasedConstraints, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.Solver.InitializeWeakConstraints"), STAT_ChaosDeformableSolver_InitializeWeakConstraints, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.Solver.InitializeKinematicConstraint"), STAT_ChaosDeformableSolver_InitializeKinematicConstraint, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.Solver.InitializeCollisionBodies"), STAT_ChaosDeformableSolver_InitializeCollisionBodies, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.Solver.UpdateCollisionBodies"), STAT_ChaosDeformableSolver_UpdateCollisionBodies, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.Solver.InitializeConstraintBodies"), STAT_ChaosDeformableSolver_InitializeConstraintBodies, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.Solver.UpdateConstraintBodies"), STAT_ChaosDeformableSolver_UpdateConstraintBodies, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.Solver.InitializeSelfCollisionVariables"), STAT_ChaosDeformableSolver_InitializeSelfCollisionVariables, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.Solver.InitializeGridBasedConstraintVariables"), STAT_ChaosDeformableSolver_InitializeGridBasedConstraintVariables, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.Solver.InitializeGaussSeidelConstraintVariables"), STAT_ChaosDeformableSolver_InitializeGaussSeidelConstraintVariables, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.Solver.InitializeGaussSeidelConstraint"), STAT_ChaosDeformableSolver_InitializeGaussSeidelConstraint, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.Solver.InitializeMuscleActivation"), STAT_ChaosDeformableSolver_InitializeMuscleActivation, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.Solver.RemoveSimulationObjects"), STAT_ChaosDeformableSolver_RemoveSimulationObjects, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.Solver.RemoveProxy"), STAT_ChaosDeformableSolver_RemoveProxy, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.Solver.AddProxy"), STAT_ChaosDeformableSolver_AddProxy, STATGROUP_Chaos);


DEFINE_LOG_CATEGORY_STATIC(LogChaosDeformableSolver, Log, All);
namespace Chaos::Softs
{
	FDeformableDebugParams GDeformableDebugParams;

	FAutoConsoleVariableRef CVarDeformableDebugParamsDrawTetrahedralParticles(TEXT("p.Chaos.DebugDraw.Deformable.TetrahedralParticle"), GDeformableDebugParams.bDoDrawTetrahedralParticles, TEXT("Debug draw the deformable solvers tetrahedron. [def: false]"));
	FAutoConsoleVariableRef CVarDeformableDebugParamsDrawKinematicParticles(TEXT("p.Chaos.DebugDraw.Deformable.KinematicParticle"), GDeformableDebugParams.bDoDrawKinematicParticles, TEXT("Debug draw the deformables kinematic particles. [def: false]"));
	FAutoConsoleVariableRef CVarDeformableDebugParamsDrawTransientKinematicParticles(TEXT("p.Chaos.DebugDraw.Deformable.TransientKinematicParticle"), GDeformableDebugParams.bDoDrawTransientKinematicParticles, TEXT("Debug draw the deformables transient kinematic particles. [def: false]"));
	FAutoConsoleVariableRef CVarDeformableDebugParamsDrawRigidCollisionGeometry(TEXT("p.Chaos.DebugDraw.Deformable.RigidCollisionGeometry"), GDeformableDebugParams.bDoDrawRigidCollisionGeometry, TEXT("Debug draw the deformable solvers rigid collision geometry. [def: false]"));
	FAutoConsoleVariableRef CVarDeformableDebugParamsDrawParticleRadius(TEXT("p.Chaos.DebugDraw.Deformable.ParticleRadius"), GDeformableDebugParams.ParticleRadius, TEXT("Drawn kinematic particle radius. [def: 5]"));

	FDeformableXPBDCorotatedParams GDeformableXPBDCorotatedParams;
	FAutoConsoleVariableRef CVarDeformableXPBDCorotatedBatchSize(TEXT("p.Chaos.Deformable.XPBDBatchSize"), GDeformableXPBDCorotatedParams.XPBDCorotatedBatchSize, TEXT("Batch size for physics parallel for. [def: 5]"));
	FAutoConsoleVariableRef CVarDeformableXPBDCorotatedBatchThreshold(TEXT("p.Chaos.Deformable.XPBDBatchThreshold"), GDeformableXPBDCorotatedParams.XPBDCorotatedBatchThreshold, TEXT("Batch threshold for physics parallel for. [def: 5]"));
	FAutoConsoleVariableRef CVarDeformableXPBDCorotatedBatchNumLogExtremeParticle(TEXT("p.Chaos.Deformable.NumLogExtremeParticle"), GDeformableXPBDCorotatedParams.NumLogExtremeParticle, TEXT("Number of most deformed particles logged. [def: 0]"));

	FDeformableXPBDWeightedSpringConstraintParams GDeformableXPBDWeakConstraintParams;
	FAutoConsoleVariableRef CVarDeformableXPBDWeakConstraintLineWidth(TEXT("p.Chaos.Deformable.XPBDWeakConstraintLineWidth"), GDeformableXPBDWeakConstraintParams.DebugLineWidth, TEXT("Line width for visualizing the double bindings in XPBD weak constraints. [def: 5]"));
	FAutoConsoleVariableRef CVarDeformableXPBDWeakConstraintParticleWidth(TEXT("p.Chaos.Deformable.XPBDWeakConstraintParticleWidth"), GDeformableXPBDWeakConstraintParams.DebugParticleWidth, TEXT("Line width for visualizing the double bindings in XPBD weak constraints. [def: 20]"));
	FAutoConsoleVariableRef CVarDeformableXPBDWeakConstraintDebugDraw(TEXT("p.Chaos.Deformable.XPBDWeakConstraintEnableDraw"), GDeformableXPBDWeakConstraintParams.bVisualizeBindings, TEXT("Debug draw the double bindings in XPBD weak constraints. [def: false]"));

	FCriticalSection FDeformableSolver::InitializationMutex;
	FCriticalSection FDeformableSolver::RemovalMutex;
	FCriticalSection FDeformableSolver::PackageOutputMutex;
	FCriticalSection FDeformableSolver::PackageInputMutex;
	FCriticalSection FDeformableSolver::PackageRestartMutex;
	FCriticalSection FDeformableSolver::SolverEnabledMutex;

	int32 GSParallelMax = 100;
	FAutoConsoleVariableRef CVarDeformableGSParrelMax(TEXT("p.Chaos.Deformable.GSParallelMax"), GSParallelMax, TEXT("Minimal number of particles to process in parallel for Gauss Seidel constraints. [def: 100]"));

	float MaxDxRatio = 1.f;
	FAutoConsoleVariableRef CVarDeformableGSMaxDxRatio(TEXT("p.Chaos.Deformable.GSMaxDxRatio"), MaxDxRatio, TEXT("Max size for dx in each iteration for Gauss Seidel constraints. [def: 1]"));

	FDeformableSolver::FDeformableSolver(FDeformableSolverProperties InProp)
		: CurrentInputPackage(TUniquePtr < FDeformablePackage >(nullptr))
		, PreviousInputPackage(TUniquePtr < FDeformablePackage >(nullptr))
		, Property(InProp)
	{
		PERF_SCOPE(STAT_ChaosDeformableSolver_Constructor);
		Reset(Property);
	}

	FDeformableSolver::~FDeformableSolver()
	{
		PERF_SCOPE(STAT_ChaosDeformableSolver_Destructor);

		FScopeLock Lock(&InitializationMutex);
		for (FThreadingProxy* Proxy : UninitializedProxys_Internal)
		{
			delete Proxy;
		}
		UninitializedProxys_Internal.Empty();
		EventTeardown.Broadcast();
	}


	void FDeformableSolver::Reset(const FDeformableSolverProperties& InProps)
	{
		PERF_SCOPE(STAT_ChaosDeformableSolver_Reset);

		Property = InProps;
		MObjects = TArrayCollectionArray<const UObject*>();
		FSolverParticles LocalParticlesDummy;
		FSolverCollisionParticles RigidParticles;
		Evolution.Reset(new FPBDEvolution(MoveTemp(LocalParticlesDummy), MoveTemp(RigidParticles), {},
			Property.NumSolverIterations, (FSolverReal)0.,
			/*SelfCollisionsThickness = */(FSolverReal)0.,
			/*CoefficientOfFriction = */(FSolverReal)0.,
			/*FSolverReal Damping = */(FSolverReal)0.,
			/*FSolverReal LocalDamping = */(FSolverReal)0.,
			Property.bDoQuasistatics, 
			true));
		Evolution->Particles().AddArray(&MObjects);
		if (Property.bDoSpringCollision || Property.bDoSphereRepulsion || Property.CacheToFile)
		{
			SurfaceElements.Reset(new TArray<Chaos::TVec3<int32>>());
			TetmeshSurfaceElements.Reset(new TArray<Chaos::TVec3<int32>>());
		}

		if (Property.bDoSpringCollision || Property.bDoSphereRepulsion)
		{
			ParticleComponentIndex.Reset(new TArray<int32>());
			SurfaceTriangleMesh.Reset(new Chaos::FTriangleMesh());
			SurfaceCollisionVertices.Reset(new TArray<int32>);
		}
		if (Property.bUseGridBasedConstraints)
		{
			AllElements.Reset(new TArray<Chaos::TVec4<int32>>());
		}
		if (Property.bUseGaussSeidelConstraints)
		{
			AllElements.Reset(new TArray<Chaos::TVec4<int32>>());
			AllIncidentElements.Reset(new TArray<TArray<int32>>());
			AllIncidentElementsLocal.Reset(new TArray<TArray<int32>>());
			AllTetEMeshArray.Reset(new TArray<FSolverReal>());
			AllTetNuMeshArray.Reset(new TArray<FSolverReal>());
			AllTetAlphaJArray.Reset(new TArray<FSolverReal>());
			AllCorotatedCodEMeshArray.Reset(new TArray<FSolverReal>());
			AllSkinEMeshArray.Reset(new TArray<FSolverReal>());
			GSWeakConstraints.Reset(new FGaussSeidelSpringConstraints({}, {}, {}, {}, {}, {}, {}, GDeformableXPBDWeakConstraintParams));
			GSDynamicWeakConstraints.Reset(new FGaussSeidelDynamicSpringConstraints(GDeformableXPBDWeakConstraintParams));
			GSSphereRepulsionConstraints.Reset(new FGaussSeidelParticleSphereRepulsionConstraints(Property.SphereRepulsionRadius, Property.SphereRepulsionStiffness, Evolution->Particles(), GDeformableXPBDWeakConstraintParams));
			GSVolumeConstraints.Reset(new FGaussSeidelUnilateralVolumeConstraints(Evolution->Particles(), {}, {}));
			MuscleIndexOffset.Empty();
			MuscleActivationConstraints.Reset(new FMuscleActivationTetrahedralConstraints());
		}
		AllUnconstrainedSurfaceElementsCorotatedCod.Reset(new TArray<Chaos::TVec3<int32>>());
		AllUnconstrainedSurfaceElementsSkin.Reset(new TArray<Chaos::TVec3<int32>>());

		InitializeKinematicConstraint();
		Frame = 0;
		Time = 0.f;
		Iteration = 0;

		// Add a default floor the first time through
		if (Property.bUseFloor)
		{
			Chaos::FVec3 Position(0.f);
			Chaos::FVec3 EulerRot(0.f);
			int32 CollisionParticleOffset = Evolution->AddCollisionParticleRange(1, INDEX_NONE, true);
			Evolution->CollisionParticles().SetX(0, Position);
			Evolution->CollisionParticles().SetR(0, Chaos::TRotation<Chaos::FReal, 3>::MakeFromEuler(EulerRot));
			Evolution->CollisionParticles().SetGeometry(0, MakeImplicitObjectPtr<Chaos::TPlane<Chaos::FReal, 3>>(Chaos::FVec3(0.f, 0.f, 0.f), Chaos::FVec3(0.f, 0.f, 1.f)));
		}
	}

	void FDeformableSolver::LoadRestartData()
	{
		// pull CurrentRestartPackage. This is after UDeformableSolverComponent::ReadRestartData() that pushes it.
		UpdateProxyRestartPackages();
		// update evolution particles
		UpdateRestartParticlePositions();
	}

	void FDeformableSolver::UpdateProxyRestartPackages()
	{
		TUniquePtr < FDeformablePackage > TailPackage = PullRestartPackage();
		while (TailPackage)
		{
			CurrentRestartPackage = TUniquePtr < FDeformablePackage >(TailPackage.Release());
			TailPackage = PullRestartPackage();
		}
	}

	void FDeformableSolver::UpdateRestartParticlePositions()
	{
		for (const TPair<FThreadingProxy::FKey, TUniquePtr<FThreadingProxy>>& Entry : Proxies)
		{
			const FThreadingProxy::FKey& Owner = Entry.Key;
			if (const FFleshThreadingProxy* Proxy = Entry.Value->As<FFleshThreadingProxy>())
			{
				if (this->CurrentRestartPackage)
				{
					if (FFleshThreadingProxy::FFleshRestartBuffer* FleshRestartBuffer =
						this->CurrentRestartPackage->ObjectMap.Contains(Owner) ?
						this->CurrentRestartPackage->ObjectMap[Owner]->As<FFleshThreadingProxy::FFleshRestartBuffer>() :
						nullptr)
					{
						if (const TManagedArray<FVector3f>* DynamicVertex = FleshRestartBuffer->Dynamic.FindAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup))
						{
							const Chaos::FRange& Range = Proxy->GetSolverParticleRange();
							for (int32 i = Range.Start; i < Range.Start + Range.Count; ++i)
							{
								Evolution->Particles().SetX(i, (*DynamicVertex)[i]);
							}							
						} // if has DynamicVertex
					} // if flesh input buffer
				} // if CurrentRestartPackage
			}
		} // for all proxies
	}
	void FDeformableSolver::Simulate(FSolverReal DeltaTime)
	{
		PERF_SCOPE(STAT_ChaosDeformableSolver_Simulate);
		if (Property.NumSolverIterations)
		{
			RemoveSimulationObjects();
			UpdateProxyInputPackages();
			InitializeSimulationObjects();
			InitializeSimulationSpace();
			if (bPendingRestart)
			{
				LoadRestartData();
				bPendingRestart = false;
			}
			AdvanceDt(DeltaTime);
			DebugDrawSimulationData();
		}
	}

	void FDeformableSolver::UpdateTransientConstraints()
	{
		for(TMap<FThreadingProxy::FKey, TUniquePtr<FThreadingProxy>>::TConstIterator ProxyIt=Proxies.CreateConstIterator(); ProxyIt; ++ProxyIt)
		{
			const FThreadingProxy::FKey& Owner = ProxyIt.Key();
			if (const FFleshThreadingProxy* Proxy = ProxyIt.Value()->As<FFleshThreadingProxy>())
			{
				if(this->CurrentInputPackage)
				{
					if (FFleshThreadingProxy::FFleshInputBuffer* FleshInputBuffer =
						this->CurrentInputPackage->ObjectMap.Contains(Owner) ?
						this->CurrentInputPackage->ObjectMap[Owner]->As<FFleshThreadingProxy::FFleshInputBuffer>() :
						nullptr)
					{
						GeometryCollection::Facades::FConstraintOverrideTargetFacade CnstrTargets(FleshInputBuffer->SimulationCollection);
						if (CnstrTargets.IsValid() && CnstrTargets.Num())
						{
							const Chaos::FRange& Range = Proxy->GetSolverParticleRange();
							const FSolverReal CurrentRatio = FSolverReal(this->Iteration) / FSolverReal(this->Property.NumSolverSubSteps);
							const FTransform WorldToSim = Proxy->GetCurrentPointsTransform();

							if (this->Iteration == 1)
							{
								TransientConstraintBuffer.Reserve(TransientConstraintBuffer.Num() + CnstrTargets.Num());
								for (int32 i = 0; i < CnstrTargets.Num(); i++)
								{
									int32 LocalIndex = CnstrTargets.GetIndex(i);
									int32 ParticleIndex = Range.Start + LocalIndex;
									// Set particle kinematic state to kinematic, saving prior state.
									TransientConstraintBuffer.Add(
										TPair<int32, TTuple<float, float, FVector3f>>(
											ParticleIndex,
											TTuple<float, float, FVector3f>(
												Evolution->Particles().InvM(ParticleIndex),
												Evolution->Particles().PAndInvM(ParticleIndex).InvM,
												Evolution->Particles().GetX(ParticleIndex))));

									Evolution->Particles().InvM(ParticleIndex) = 0.f;
									Evolution->Particles().PAndInvM(ParticleIndex).InvM = 0.f;
								}
							}

							auto ToDouble = [](FVector3f V) { return FVector(V[0], V[1], V[2]); };
							auto ToSingle = [](FVector V) { return FVector3f(static_cast<float>(V[0]), static_cast<float>(V[1]), static_cast<float>(V[2])); };

							for (int32 i = 0; i < CnstrTargets.Num(); i++)
							{
								int32 LocalIndex = CnstrTargets.GetIndex(i);
								int32 ParticleIndex = Range.Start + LocalIndex;

								const FVector3f& WorldSpaceTarget = CnstrTargets.GetPosition(i);
								FVector3f SimSpaceTarget = ToSingle(WorldToSim.TransformPosition(ToDouble(WorldSpaceTarget)));
								const FVector3f& SimSpaceSource = TransientConstraintBuffer[ParticleIndex].Get<2>();

								// Lerp from previous particle position to the target over the solver iterations.
								Evolution->Particles().SetX(ParticleIndex,
									SimSpaceTarget * CurrentRatio +
									SimSpaceSource * (static_cast<FSolverReal>(1.) - CurrentRatio));
								Evolution->Particles().PAndInvM(ParticleIndex).P =
									Evolution->Particles().GetX(ParticleIndex);
							}
#if WITH_EDITOR
							if (GDeformableDebugParams.IsDebugDrawingEnabled() && GDeformableDebugParams.bDoDrawTransientKinematicParticles)
							{
								auto DoubleVert = [](FVector3f V) { return FVector3d(V.X, V.Y, V.Z); };
								for (int32 i = 0; i < CnstrTargets.Num(); i++)
								{
									int32 LocalIndex = CnstrTargets.GetIndex(i);
									int32 ParticleIndex = Range.Start + LocalIndex;
									Chaos::FDebugDrawQueue::GetInstance().DrawDebugPoint(ToDouble(Evolution->Particles().GetX(ParticleIndex)), FColor::Orange, false, -1.0f, 0, GDeformableDebugParams.ParticleRadius);
								}
							}
#endif
						} // if has constraint overrides
					} // if flesh input buffer
				}
			}
		} // for all proxies
	}

	void FDeformableSolver::PostProcessTransientConstraints()
	{
		// Restore transient constraint particle kinematic state.
		if (!TransientConstraintBuffer.IsEmpty())
		{
			for (TransientConstraintBufferMap::TConstIterator It = TransientConstraintBuffer.CreateConstIterator(); It; ++It)
			{
				const int32 ParticleIndex = It.Key();
				Evolution->Particles().InvM(ParticleIndex) = It.Value().Get<0>();
				Evolution->Particles().PAndInvM(ParticleIndex).InvM = It.Value().Get<1>();
			}
			TransientConstraintBuffer.Reset(); // retains memory
		}
	}

	void FDeformableSolver::InitializeSimulationSpace()
	{
		for (int32 Index = 0; Index < this->MObjects.Num(); Index++)
		{
			if (const UObject* Owner = this->MObjects[Index])
			{
				if (FFleshThreadingProxy* Proxy = Proxies[Owner]->As<FFleshThreadingProxy>())
				{
					if (this->CurrentInputPackage && this->CurrentInputPackage->ObjectMap.Contains(Owner))
					{
						FFleshThreadingProxy::FFleshInputBuffer* FleshInputBuffer =
							this->CurrentInputPackage->ObjectMap[Owner]->As<FFleshThreadingProxy::FFleshInputBuffer>();
						if (FleshInputBuffer)
						{
							Proxy->UpdateSimSpace(FleshInputBuffer->WorldToComponentXf, FleshInputBuffer->ComponentToBoneXf);
						}
					}
				}
			}
		}
	}

	void FDeformableSolver::InitializeSimulationObjects()
	{
		PERF_SCOPE(STAT_ChaosDeformableSolver_InitializeSimulationObjects);
		{
			FScopeLock Lock(&InitializationMutex); // @todo(flesh) : change to threaded task based commands to prevent the lock. 
			if (UninitializedProxys_Internal.Num())
			{
				for (FThreadingProxy* Proxy : UninitializedProxys_Internal)
				{
					InitializeSimulationObject(*Proxy);

					FThreadingProxy::FKey Key = Proxy->GetOwner();
					Proxies.Add(Key, TUniquePtr<FThreadingProxy>(Proxy));
				}

				PrevEvolutionActiveRange = Evolution->ParticlesActiveView().GetActiveRanges();

				if (UninitializedProxys_Internal.Num() != 0)
				{
					if (Property.bDoSpringCollision || Property.bDoSphereRepulsion)
					{
						InitializeSelfCollisionVariables();
					}

					if (Property.bUseGridBasedConstraints)
					{
						InitializeGridBasedConstraintVariables();
					}

					if (Property.bUseGaussSeidelConstraints)
					{
						InitializeGaussSeidelConstraintVariables();
						// TODO(flesh): Support muscle activation with XPBD, use an array for GSConstraints
						InitializeMuscleActivationVariables();
					}
				}
				UninitializedProxys_Internal.SetNum(0, EAllowShrinking::Yes);
			}
		}
	}

	void FDeformableSolver::UpdateSimulationObjects(FSolverReal DeltaTime)
	{
		PERF_SCOPE(STAT_ChaosDeformableSolver_InitializeSimulationObjects);

		typedef TPair< FThreadingProxy::FKey, TUniquePtr<FThreadingProxy> > FType;
		for (const FType& Entry : Proxies)
		{
			if (Entry.Value)
			{
				FThreadingProxy& Proxy = *Entry.Value.Get();
				if (FCollisionManagerProxy* CollisionManagerProxy = Proxy.As< FCollisionManagerProxy>())
				{
					UpdateCollisionBodies(*CollisionManagerProxy, Entry.Key, DeltaTime);
				}
				else if (FConstraintManagerProxy* ConstraintManagerProxy = Proxy.As< FConstraintManagerProxy>())
				{
					UpdateConstraintBodies(*ConstraintManagerProxy, Entry.Key, DeltaTime);
				}
			}
		}

		UpdateTransientConstraints();
	}

	void FDeformableSolver::InitializeSimulationObject(FThreadingProxy& InProxy)
	{
		PERF_SCOPE(STAT_ChaosDeformableSolver_InitializeSimulationObject);

		if (FFleshThreadingProxy* Proxy = InProxy.As<FFleshThreadingProxy>())
		{
			if (Proxy->CanSimulate())
			{
				if (Proxy->GetRestCollection().NumElements(FGeometryCollection::VerticesGroup))
				{
					InitializeDeformableParticles(*Proxy);
					InitializeKinematicParticles(*Proxy);
					InitializeWeakConstraint(*Proxy);
					InitializeMuscleActivation(*Proxy);
					InitializeTetrahedralOrTriangleConstraint(*Proxy);
					InitializeGridBasedConstraints(*Proxy);
					InitializeGaussSeidelConstraints(*Proxy);
				}
			}
		}

		if (FCollisionManagerProxy* CollisionManagerProxy = InProxy.As< FCollisionManagerProxy>())
		{
			InitializeCollisionBodies(*CollisionManagerProxy);
		}

		if (FConstraintManagerProxy* ConstraintManagerProxy = InProxy.As< FConstraintManagerProxy>())
		{
			InitializeConstraintBodies(*ConstraintManagerProxy);
		}
	}

	void FDeformableSolver::InitializeDeformableParticles(FFleshThreadingProxy& Proxy)
	{
		PERF_SCOPE(STAT_ChaosDeformableSolver_InitializeDeformableParticles);

		const FManagedArrayCollection& Dynamic = Proxy.GetDynamicCollection();
		const FManagedArrayCollection& Rest = Proxy.GetRestCollection();

		const TManagedArray<FVector3f>& DynamicVertex = Dynamic.GetAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup);
		const TManagedArray<FSolverReal>* MassArray = Rest.FindAttribute<FSolverReal>("Mass", FGeometryCollection::VerticesGroup);
		const TManagedArray<FSolverReal>* DampingArray = Rest.FindAttribute<FSolverReal>("Damping", FGeometryCollection::VerticesGroup);
		FSolverReal Mass = 100.0;// @todo(chaos) : make user attributes

		auto ChaosVert = [](FVector3d V) { return Chaos::FVec3(V.X, V.Y, V.Z); };
		auto ChaosM = [](FSolverReal M, const TManagedArray<float>* AM, int32 Index, int32 Num) { return FSolverReal((AM != nullptr) ? (*AM)[Index] : M / FSolverReal(Num)); };
		auto ChaosInvM = [](FSolverReal M) { return FSolverReal(FMath::IsNearlyZero(M) ? 0.0 : 1 / M); };
		auto DoubleVert = [](FVector3f V) { return FVector3d(V.X, V.Y, V.Z); };
		uint32 NumParticles = Rest.NumElements(FGeometryCollection::VerticesGroup);
		int32 ParticleStart = Evolution->AddParticleRange(NumParticles, GroupOffset, true);
		GroupOffset += 1;
		for (uint32 vdx = 0; vdx < NumParticles; ++vdx)
		{
			MObjects[ParticleStart + vdx] = Proxy.GetOwner();
		}

		TArray<FSolverReal> MassWithMultiplier;
		TArray<FSolverReal> DampingWithMultiplier;
		MassWithMultiplier.Init(0.f, NumParticles);
		DampingWithMultiplier.Init(0.f, NumParticles);
		FSolverReal DampingMultiplier = 0.f;
		FSolverReal MassMultiplier = 0.f;
		if (const UObject* Owner = this->MObjects[ParticleStart]) {
			FFleshThreadingProxy::FFleshInputBuffer* FleshInputBuffer = nullptr;
			if (this->CurrentInputPackage->ObjectMap.Contains(Owner))
			{
				FleshInputBuffer = this->CurrentInputPackage->ObjectMap[Owner]->As<FFleshThreadingProxy::FFleshInputBuffer>();
				if (FleshInputBuffer)
				{
					DampingMultiplier = FleshInputBuffer->DampingMultiplier;
					MassMultiplier = FleshInputBuffer->MassMultiplier;
				}
			}
		}

		for (uint32 vdx = 0; vdx < NumParticles; ++vdx)
		{
			MassWithMultiplier[vdx] = ChaosM(Mass, MassArray, vdx, NumParticles) * MassMultiplier;
			if (DampingArray)
			{
				Evolution->SetParticleDamping((*DampingArray)[vdx], ParticleStart + vdx);
			}
		}

		Evolution->SetDamping(DampingMultiplier, GroupOffset - 1);


		// Tet mesh points are in component space.  That means that if our sim space is:
		//    World:     We need to multiply by the ComponentToWorldXf.
		//	  Component: We do nothing.
		//    Bone:      The points have the (component relative) bone transform baked in,
		//               and we need to remove it.  Muliply by the BoneToComponentXf.

		const FTransform& InitialPointsXf = Proxy.GetInitialPointsTransform();
		if (!InitialPointsXf.Equals(FTransform::Identity))
		{
			for (uint32 vdx = 0; vdx < NumParticles; ++vdx)
			{
				int32 SolverParticleIndex = ParticleStart + vdx;
				Evolution->Particles().SetX(SolverParticleIndex, ChaosVert(InitialPointsXf.TransformPosition(DoubleVert(DynamicVertex[vdx]))));
				Evolution->Particles().V(SolverParticleIndex) = Chaos::FVec3(0.f, 0.f, 0.f);
				Evolution->Particles().M(SolverParticleIndex) = MassWithMultiplier[vdx];
				Evolution->Particles().InvM(SolverParticleIndex) = ChaosInvM(Evolution->Particles().M(SolverParticleIndex));
				Evolution->Particles().PAndInvM(SolverParticleIndex).InvM = Evolution->Particles().InvM(SolverParticleIndex);
			}
		}
		else
		{
			for (uint32 vdx = 0; vdx < NumParticles; ++vdx)
			{
				int32 SolverParticleIndex = ParticleStart + vdx;
				Evolution->Particles().SetX(SolverParticleIndex, DynamicVertex[vdx]);
				Evolution->Particles().V(SolverParticleIndex) = Chaos::FVec3(0.f, 0.f, 0.f);
				Evolution->Particles().M(SolverParticleIndex) = MassWithMultiplier[vdx];
				Evolution->Particles().InvM(SolverParticleIndex) = ChaosInvM(Evolution->Particles().M(SolverParticleIndex));
				Evolution->Particles().PAndInvM(SolverParticleIndex).InvM = Evolution->Particles().InvM(SolverParticleIndex);
			}
		}

		bool ObjectEnableGravity = false;
		if (const UObject* Owner = this->MObjects[ParticleStart])
		{
			FFleshThreadingProxy::FFleshInputBuffer* FleshInputBuffer = nullptr;
			if (this->CurrentInputPackage->ObjectMap.Contains(Owner))
			{
				FleshInputBuffer = this->CurrentInputPackage->ObjectMap[Owner]->As<FFleshThreadingProxy::FFleshInputBuffer>();
				if (FleshInputBuffer)
				{
					ObjectEnableGravity = FleshInputBuffer->bEnableGravity;
				}
			}
		}

		if (!ObjectEnableGravity || !Property.bEnableGravity)
		{
			FSolverVec3 ZeroGravity(0.f);
			Evolution->SetGravity(ZeroGravity, GroupOffset - 1);
		}
		else
		{
			// Gravity points "down" in world space, but we need to orient it to
			// whatever our sim space is.
			FSolverVec3 GravityDir = Evolution->GetGravity();
			FSolverVec3 SimSpaceGravityDir = Proxy.RotateWorldSpaceVector(GravityDir);
			Evolution->SetGravity(SimSpaceGravityDir, GroupOffset - 1);
		}

		Proxy.SetSolverParticleRange(ParticleStart, NumParticles);
	}

	void FDeformableSolver::InitializeKinematicParticles(FFleshThreadingProxy& Proxy)
	{
		PERF_SCOPE(STAT_ChaosDeformableSolver_InitializeKinematicParticles);

		const FManagedArrayCollection& Rest = Proxy.GetRestCollection();
		const Chaos::FRange& Range = Proxy.GetSolverParticleRange();

		if (Property.bEnableKinematics)
		{
			GeometryCollection::Facades::FVertexBoneWeightsFacade VertexBoneWeightsFacade(Rest);
			for (int32 VertexIdx = 0; VertexIdx < VertexBoneWeightsFacade.NumVertices(); ++VertexIdx)
			{
				if (VertexBoneWeightsFacade.IsKinematicVertex(VertexIdx))
				{
					int32 ParticleIndex = Range.Start + VertexIdx;
					Evolution->Particles().InvM(ParticleIndex) = 0.f;
					Evolution->Particles().PAndInvM(ParticleIndex).InvM = 0.f;
				}
			}
			//Supports backward compatibility for pre-5.5 nodes that uses bone-based bindings
			//To be removed post-5.6
			typedef GeometryCollection::Facades::FKinematicBindingFacade FKinematics;
			FKinematics Kinematics(Rest);
			if (Kinematics.IsValid())
			{
				// Add Kinematics Node
				bool bHavePrintedLog = false;
				for (int i = Kinematics.NumKinematicBindings() - 1; i >= 0; i--)
				{
					FKinematics::FBindingKey Key = Kinematics.GetKinematicBindingKey(i);

					int32 BoneIndex = INDEX_NONE;
					TArray<int32> BoundVerts;
					TArray<float> BoundWeights;
					Kinematics.GetBoneBindings(Key, BoneIndex, BoundVerts, BoundWeights);

					for (int32 vdx : BoundVerts)
					{
						if (vdx >= 0 && !VertexBoneWeightsFacade.IsKinematicVertex(vdx))
						{
							if (!bHavePrintedLog)
							{
								bHavePrintedLog = true;
								UE_LOGF(LogChaosDeformableSolver, Warning, "Detected deprecated kinematic initialization, reevaluate input asset");
							}
							int32 ParticleIndex = Range.Start + vdx;
							Evolution->Particles().InvM(ParticleIndex) = 0.f;
							Evolution->Particles().PAndInvM(ParticleIndex).InvM = 0.f;
						}
					}
				}
			}

		}
	}

	void FDeformableSolver::InitializeWeakConstraint(FFleshThreadingProxy& Proxy)
	{
		PERF_SCOPE(STAT_ChaosDeformableSolver_InitializeWeakConstraints);

		const FManagedArrayCollection& Rest = Proxy.GetRestCollection();
		const Chaos::FRange& Range = Proxy.GetSolverParticleRange();
		if (Property.bEnablePositionTargets)
		{
			typedef GeometryCollection::Facades::FPositionTargetFacade FPositionTargets;
			FPositionTargets PositionTargets(Rest);

			TArray<TArray<int32>> PositionTargetIndices;
			TArray<TArray<FSolverReal>> PositionTargetWeights;
			TArray<TArray<int32>> PositionTargetSecondIndices;
			TArray<TArray<FSolverReal>> PositionTargetSecondWeights;
			TArray<FSolverReal> PositionTargetStiffness;
			TArray<bool> PositionTargetIsAnisotrpic;
			TArray<bool> PositionTargetIsZeroRestLength;
			int32 NumPositionTargets = PositionTargets.NumPositionTargets();
			PositionTargetIndices.SetNum(NumPositionTargets);
			PositionTargetWeights.SetNum(NumPositionTargets);
			PositionTargetSecondIndices.SetNum(NumPositionTargets);
			PositionTargetSecondWeights.SetNum(NumPositionTargets);
			PositionTargetStiffness.SetNum(NumPositionTargets);
			PositionTargetIsAnisotrpic.SetNum(NumPositionTargets);
			PositionTargetIsZeroRestLength.SetNum(NumPositionTargets);

			// Read in position target info
			for (int i = NumPositionTargets - 1; i >= 0; i--)
			{
				GeometryCollection::Facades::FPositionTargetsData DataPackage = PositionTargets.GetPositionTarget(i);
				PositionTargetIndices[i] = DataPackage.SourceIndex;
				PositionTargetWeights[i] = DataPackage.SourceWeights;
				PositionTargetSecondIndices[i] = DataPackage.TargetIndex;
				PositionTargetSecondWeights[i] = DataPackage.TargetWeights;
				PositionTargetStiffness[i] = DataPackage.Stiffness;
				PositionTargetIsAnisotrpic[i] = DataPackage.bIsAnisotropic;
				PositionTargetIsZeroRestLength[i] = DataPackage.bIsZeroRestLength;
			}

			if (Property.bUseGaussSeidelConstraints)
			{
				for (int i = PositionTargets.NumPositionTargets() - 1; i >= 0; i--)
				{
					for (int32 j = 0; j < PositionTargetIndices[i].Num(); j++)
					{
						PositionTargetIndices[i][j] += Range.Start;
					}
					for (int32 j = 0; j < PositionTargetSecondIndices[i].Num(); j++)
					{
						PositionTargetSecondIndices[i][j] += Range.Start;
					}
				}
				GSWeakConstraints->AddExtraConstraints(PositionTargetIndices, PositionTargetWeights, 
					PositionTargetStiffness, PositionTargetSecondIndices, PositionTargetSecondWeights, 
					PositionTargetIsAnisotrpic, PositionTargetIsZeroRestLength);
			} 
			else
			{
				int32 InitIndex = Evolution->AddConstraintInitRange(1, true);
				int32 ConstraintIndex = Evolution->AddConstraintRuleRange(1, true);

				FXPBDWeightedSpringConstraints* WeakConstraint =
					new FXPBDWeightedSpringConstraints(Evolution->Particles(),
						PositionTargetIndices, PositionTargetWeights, PositionTargetStiffness, PositionTargetSecondIndices, PositionTargetSecondWeights, GDeformableXPBDWeakConstraintParams);

				Evolution->ConstraintInits()[InitIndex] =
					[WeakConstraint, this](FSolverParticles& InParticles, const FSolverReal Dt)
				{
					WeakConstraint->Init(InParticles, Dt);
				};

				Evolution->ConstraintRules()[ConstraintIndex] =
					[WeakConstraint](FSolverParticles& InParticles, const FSolverReal Dt)
				{
					WeakConstraint->ApplyInParallel(InParticles, Dt);
				};

				WeakConstraints.Add(TUniquePtr<FXPBDWeightedSpringConstraints>(WeakConstraint));
			}
		}

		//initialize volume constraint
		GeometryCollection::Facades::FVolumeConstraintFacade VolumeConstraint(Rest);
		int32 NumConstraints = VolumeConstraint.NumVolumeConstraints();
		TArray<TVector<int32, 4>> InConstraints;
		TArray<FSolverReal> InVolumes;
		TArray<FSolverReal> InStiffnessArray;
		InConstraints.SetNumUninitialized(NumConstraints);
		InStiffnessArray.SetNumUninitialized(NumConstraints);
		for (int32 ConstraintIdx = 0; ConstraintIdx < NumConstraints; ++ConstraintIdx)
		{
			for (int32 LocalIdx = 0; LocalIdx < 4; ++LocalIdx)
			{
				InConstraints[ConstraintIdx][LocalIdx] = VolumeConstraint.GetVolumeIndex(ConstraintIdx)[LocalIdx];
			}
			InStiffnessArray[ConstraintIdx] = VolumeConstraint.GetStiffness(ConstraintIdx);
		}
		GSVolumeConstraints.Reset(new FGaussSeidelUnilateralVolumeConstraints(Evolution->Particles(), MoveTemp(InConstraints), MoveTemp(InStiffnessArray)));
		if (GSVolumeConstraints->NumConstraints() && !Property.bUseGaussSeidelConstraints)
		{
			UE_LOGF(LogChaosDeformableSolver, Error, "Error: must check [Use Gauss Seidel constraints] for volume constraints.");
		}
	}


	void FDeformableSolver::InitializeCollisionBodies(FCollisionManagerProxy& Proxy)
	{
		PERF_SCOPE(STAT_ChaosDeformableSolver_InitializeCollisionBodies);
	}

	void FDeformableSolver::UpdateCollisionBodies(FCollisionManagerProxy& Proxy, FThreadingProxy::FKey Owner, FSolverReal DeltaTime)
	{
		PERF_SCOPE(STAT_ChaosDeformableSolver_UpdateCollisionBodies);

		FCollisionManagerProxy::FCollisionsInputBuffer* CollisionsInputBuffer = nullptr;
		if (this->CurrentInputPackage && this->CurrentInputPackage->ObjectMap.Contains(Owner))
		{
			if (this->CurrentInputPackage->ObjectMap[Owner] != nullptr)
			{
				CollisionsInputBuffer = this->CurrentInputPackage->ObjectMap[Owner]->As<FCollisionManagerProxy::FCollisionsInputBuffer>();
				if (CollisionsInputBuffer)
				{
					TArray<FCollisionObjectAddedBodies> IgnoredAdditions;
					for (auto& AddBody : CollisionsInputBuffer->Added)
					{
						if (AddBody.Shapes)
						{
							if (!Proxy.CollisionBodies.Contains(AddBody.Key))
							{
								int32 Index = Evolution->AddCollisionParticle(INDEX_NONE, true);
								int32 ViewIndex = Evolution->CollisionParticlesActiveView().GetNumRanges() - 1;
								Evolution->CollisionParticles().SetX(Index, AddBody.Transform.GetTranslation());
								Evolution->CollisionParticles().SetR(Index, AddBody.Transform.GetRotation());
								Chaos::FImplicitObjectPtr UniquePtr(AddBody.Shapes); AddBody.Shapes = nullptr;
								Evolution->CollisionParticles().SetGeometry(Index, MoveTemp(UniquePtr));
								Proxy.CollisionBodies.Add(AddBody.Key, FCollisionObjectParticleHandel(Index, ViewIndex, AddBody.Transform));
							}
							else
							{
								IgnoredAdditions.Add(AddBody);
							}
						}
					}

					// If we tried to add a body that already was added, this means their
					// should be a matching delete as the body was removed and added
					// back before the physics thread could evaluate. 
					for (auto& AddedBody : IgnoredAdditions)
					{
						for (int i=0; i<CollisionsInputBuffer->Removed.Num();)
						{
							if ((void*)CollisionsInputBuffer->Removed[i].Key.Get<0>() == (void*)AddedBody.Key.Get<0>())
							{
								CollisionsInputBuffer->Removed.RemoveAtSwap(i);
								if (i == CollisionsInputBuffer->Removed.Num() - 1)
								{
									break;
								}
							}
							else
							{
								i++;
							}
						}
					}

					TArray<FCollisionObjectKey> KeysToRemove;
					for (auto& RemovedBody : CollisionsInputBuffer->Removed)
					{
						for (auto& CollisionBodyPair : Proxy.CollisionBodies)
						{
							if ((void*)CollisionBodyPair.Key.Get<0>() == (void*)RemovedBody.Key.Get<0>())
							{
								KeysToRemove.Add(CollisionBodyPair.Key);
							}
						}
					}
					for (auto& KeyToRemove : KeysToRemove)
					{
						if (Proxy.CollisionBodies.Contains(KeyToRemove))
						{
							int32 ParticleIndex = Proxy.CollisionBodies[KeyToRemove].ParticleIndex;
							int32 ViewIndex = Proxy.CollisionBodies[KeyToRemove].ActiveViewIndex;
							Evolution->RemoveCollisionParticle(ParticleIndex, ViewIndex);
							Proxy.CollisionBodies.Remove(KeyToRemove);
						}
					}

					// Do updates
					for (auto& UpdateBody : CollisionsInputBuffer->Updated)
					{
						if (Proxy.CollisionBodies.Contains(UpdateBody.Key))
						{
							FCollisionObjectParticleHandel* ParticleHandle = Proxy.CollisionBodies.Find(UpdateBody.Key);
							Evolution->CollisionParticles().SetX(ParticleHandle->ParticleIndex, UpdateBody.Transform.GetTranslation());
							Evolution->CollisionParticles().SetR(ParticleHandle->ParticleIndex, UpdateBody.Transform.GetRotation());
						}
					}
				}
			}
		}

	}

	void FDeformableSolver::InitializeConstraintBodies(FConstraintManagerProxy& Proxy)
	{
		PERF_SCOPE(STAT_ChaosDeformableSolver_InitializeConstraintBodies);
	}

	void FDeformableSolver::UpdateConstraintBodies(FConstraintManagerProxy& Proxy, FThreadingProxy::FKey Owner, FSolverReal DeltaTime)
	{
		PERF_SCOPE(STAT_ChaosDeformableSolver_UpdateConstraintBodies);

		FConstraintManagerProxy::FConstraintsInputBuffer* ConstraintsInputBuffer = nullptr;
		if (this->CurrentInputPackage && this->CurrentInputPackage->ObjectMap.Contains(Owner))
		{
			if (this->CurrentInputPackage->ObjectMap[Owner] != nullptr)
			{
				ConstraintsInputBuffer = this->CurrentInputPackage->ObjectMap[Owner]->As<FConstraintManagerProxy::FConstraintsInputBuffer>();
				if (ConstraintsInputBuffer)
				{
					for (auto& AddConstraints : ConstraintsInputBuffer->Added)
					{
						//@todo (defered initilziation) : Save the added constraint into a buffer for processing later. 
						UE_LOGF(LogChaosDeformableSolver, Log, "Process Constraint : %ls", *Proxy.GetOwner()->GetName());

						if (Proxies.Contains(AddConstraints.Get<0>()) && Proxies.Contains(AddConstraints.Get<1>()))
						{
							FFleshThreadingProxy* SourceProxy = Proxies[AddConstraints.Get<0>()]->As<FFleshThreadingProxy>();
							FFleshThreadingProxy* TargetProxy = Proxies[AddConstraints.Get<1>()]->As<FFleshThreadingProxy>();
							if (SourceProxy && TargetProxy)
							{
								Chaos::FRange SampleRange = SourceProxy->GetSolverParticleRange();
								TConstArrayView<Softs::FSolverVec3> Samples(&Evolution->Particles().GetX(SampleRange.Start), SampleRange.Count);

								Chaos::FRange TargetRange = TargetProxy->GetSolverParticleRange();
								TConstArrayView<Softs::FSolverVec3> TetVertices(&Evolution->Particles().GetX(TargetRange.Start), TargetRange.Count);

								const GeometryCollection::Facades::FTetrahedralFacade Geom(TargetProxy->GetRestCollection());
								TArray<GeometryCollection::Facades::TetrahedralParticleEmbedding> Intersections;
								if (Geom.Intersection(Samples, TetVertices, Intersections))
								{
									UE_LOGF(LogChaosDeformableSolver, Log, "... Intersections : %d", Intersections.Num());
									if (GSDynamicWeakConstraints && Property.bEnableDynamicSprings)
									{
										float Stiffness = AddConstraints.Parameters.Stiffness;
										TArray<const FGaussSeidelSpringConstraints::FGaussSeidelConstraintHandle*> ConstraintHandles = GSDynamicWeakConstraints -> AddParticleTetrahedraConstraints(Geom, Evolution->Particles(), Intersections, SampleRange, TargetRange, Stiffness);
										Chaos::Softs::FConstraintObjectParticleHandel& HandleValue = Proxy.Constraints.FindOrAdd(AddConstraints);
										HandleValue.Handles = ConstraintHandles;
										bDynamicConstraintIsUpdated = true;
									}
								}
							}
						}
					}
					ConstraintsInputBuffer->Added.Empty();
				}
			}
		}
	}

	void FDeformableSolver::DebugDrawTetrahedralParticles(FFleshThreadingProxy& Proxy)
	{
#if WITH_EDITOR
		auto ChaosTet = [](FIntVector4 V, int32 dp) { return Chaos::TVec4<int32>(dp + V.X, dp + V.Y, dp + V.Z, dp + V.W); };
		auto ChaosVert = [](FVector3d V) { return Chaos::FVec3(V.X, V.Y, V.Z); };
		auto DoubleVert = [](FVector3f V) { return FVector3d(V.X, V.Y, V.Z); };

		const Chaos::FRange& Range = Proxy.GetSolverParticleRange();
		const FManagedArrayCollection& Rest = Proxy.GetRestCollection();
		if (const TManagedArray<FIntVector4>* Tetrahedron = Rest.FindAttributeTyped<FIntVector4>("Tetrahedron", "Tetrahedral"))
		{
			if (uint32 NumElements = Tetrahedron->Num())
			{
				const Chaos::Softs::FSolverParticles& P = Evolution->Particles();
				for (uint32 edx = 0; edx < NumElements; ++edx)
				{
					auto T = ChaosTet((*Tetrahedron)[edx], Range.Start);
					Chaos::FDebugDrawQueue::GetInstance().DrawDebugPoint(
						DoubleVert(P.GetX(T[0])), FColor::Blue, false, -1.0f, 0, GDeformableDebugParams.ParticleRadius);
				}
			}
		}
#endif
	}


	void FDeformableSolver::InitializeTetrahedralOrTriangleConstraint(FFleshThreadingProxy& Proxy)
	{
		PERF_SCOPE(STAT_ChaosDeformableSolver_InitializeTetrahedralOrTriangleConstraint);

		const FManagedArrayCollection& Rest = Proxy.GetRestCollection();

		auto ChaosTet = [](FIntVector4 V, int32 dp) { return Chaos::TVec4<int32>(dp + V.X, dp + V.Y, dp + V.Z, dp + V.W); };

		TArray<FSolverReal> StiffnessWithMultiplier;
		const Chaos::FRange& Range = Proxy.GetSolverParticleRange();
		if (Rest.HasAttributes({ FManagedArrayCollection::TManagedType<FSolverReal>("Stiffness", FGeometryCollection::VerticesGroup) }))
		{
			uint32 NumParticles = Rest.NumElements(FGeometryCollection::VerticesGroup);

			StiffnessWithMultiplier.Init(0.f, NumParticles);
			FSolverReal StiffnessMultiplier = 1.f;
			FSolverReal IncompressibilityMultiplier = 1.f;
			FSolverReal InflationMultiplier = 1.f;

			if (const UObject* Owner = this->MObjects[Range.Start]) {
				FFleshThreadingProxy::FFleshInputBuffer* FleshInputBuffer = nullptr;
				if (this->CurrentInputPackage->ObjectMap.Contains(Owner))
				{
					FleshInputBuffer = this->CurrentInputPackage->ObjectMap[Owner]->As<FFleshThreadingProxy::FFleshInputBuffer>();
					if (FleshInputBuffer)
					{
						StiffnessMultiplier = FleshInputBuffer->StiffnessMultiplier;
					}
				}
			}
			const TManagedArray<FSolverReal>* StiffnessArray = Rest.FindAttribute<FSolverReal>("Stiffness", FGeometryCollection::VerticesGroup);
			if (StiffnessArray)
			{
				for (uint32 vdx = 0; vdx < NumParticles; ++vdx)
				{
					StiffnessWithMultiplier[vdx] = (*StiffnessArray)[vdx] * StiffnessMultiplier;
				}
			}
		}

		if (const TManagedArray<FIntVector4>* Tetrahedron = Rest.FindAttributeTyped<FIntVector4>("Tetrahedron", "Tetrahedral"))
		{
			if (uint32 NumElements = Tetrahedron->Num())
			{
				// Add Tetrahedral Elements Node
				TArray<Chaos::TVec4<int32>> Elements;
				Elements.SetNum(NumElements);
				for (uint32 edx = 0; edx < NumElements; ++edx)
				{
					Elements[edx] = ChaosTet((*Tetrahedron)[edx], Range.Start);
				}

				if (Property.bUseGridBasedConstraints)
				{
					int32 ElementsOffset = AllElements->Num();
					AllElements->SetNum(ElementsOffset + NumElements);
					for (uint32 edx = 0; edx < NumElements; ++edx)
					{
						(*AllElements)[edx + ElementsOffset] = ChaosTet((*Tetrahedron)[edx], Range.Start);
					}

				}

				if (Rest.HasAttributes({ FManagedArrayCollection::TManagedType<FSolverReal>("Stiffness", FGeometryCollection::VerticesGroup) }))
				{
					uint32 NumParticles = Rest.NumElements(FGeometryCollection::VerticesGroup);

					//StiffnessWithMultiplier.Init(0.f, NumParticles);
					FSolverReal StiffnessMultiplier = 1.f;
					FSolverReal IncompressibilityMultiplier = 1.f;
					FSolverReal InflationMultiplier = 1.f;

					if (const UObject* Owner = this->MObjects[Range.Start]) {
						FFleshThreadingProxy::FFleshInputBuffer* FleshInputBuffer = nullptr;
						if (this->CurrentInputPackage->ObjectMap.Contains(Owner))
						{
							FleshInputBuffer = this->CurrentInputPackage->ObjectMap[Owner]->As<FFleshThreadingProxy::FFleshInputBuffer>();
							if (FleshInputBuffer)
							{
								IncompressibilityMultiplier = FleshInputBuffer->IncompressibilityMultiplier;
								InflationMultiplier = FleshInputBuffer->InflationMultiplier;
							}
						}
					}
					const TManagedArray<FSolverReal>* StiffnessArray = Rest.FindAttribute<FSolverReal>("Stiffness", FGeometryCollection::VerticesGroup);
					TArray<FSolverReal> TetStiffness;
					TetStiffness.Init(0.f, Elements.Num());
					if (StiffnessArray)
					{
						for (int32 edx = 0; edx < Elements.Num(); edx++)
						{
							TetStiffness[edx] = (StiffnessWithMultiplier[(*Tetrahedron)[edx].X] + StiffnessWithMultiplier[(*Tetrahedron)[edx].Y]
								+ StiffnessWithMultiplier[(*Tetrahedron)[edx].Z] + StiffnessWithMultiplier[(*Tetrahedron)[edx].W]) / 4.f;
						}
					}

					const TManagedArray<FSolverReal>* IncompressibilityArray = Rest.FindAttribute<FSolverReal>("Incompressibility", FGeometryCollection::VerticesGroup);
					TArray<FSolverReal> TetNu, AlphaJMesh, IncompressibilityWithMultiplier, InflationWithMultiplier;
					TetNu.Init(.3f, Elements.Num());

					IncompressibilityWithMultiplier.Init(0.f, NumParticles);
					InflationWithMultiplier.Init(0.f, NumParticles);
					if (IncompressibilityArray)
					{
						for (uint32 vdx = 0; vdx < NumParticles; ++vdx)
						{
							IncompressibilityWithMultiplier[vdx] = (*IncompressibilityArray)[vdx] * IncompressibilityMultiplier;
						}
						for (int32 edx = 0; edx < Elements.Num(); edx++)
						{
							TetNu[edx] = (IncompressibilityWithMultiplier[(*Tetrahedron)[edx].X] + IncompressibilityWithMultiplier[(*Tetrahedron)[edx].Y]
								+ IncompressibilityWithMultiplier[(*Tetrahedron)[edx].Z] + IncompressibilityWithMultiplier[(*Tetrahedron)[edx].W]) / 4.f;
						}
					}

					const TManagedArray<FSolverReal>* InflationArray = Rest.FindAttribute<FSolverReal>("Inflation", FGeometryCollection::VerticesGroup);
					AlphaJMesh.Init(1.f, Elements.Num());
					if (InflationArray)
					{
						for (uint32 vdx = 0; vdx < NumParticles; ++vdx)
						{
							InflationWithMultiplier[vdx] = (*InflationArray)[vdx] * InflationMultiplier;
						}
						for (int32 edx = 0; edx < Elements.Num(); edx++)
						{
							AlphaJMesh[edx] = (InflationWithMultiplier[(*Tetrahedron)[edx].X] + InflationWithMultiplier[(*Tetrahedron)[edx].Y]
								+ InflationWithMultiplier[(*Tetrahedron)[edx].Z] + InflationWithMultiplier[(*Tetrahedron)[edx].W]) / 4.f;
						}
					}

					if (Property.bUseGaussSeidelConstraints)
					{
						int32 ElementsOffset = AllTetEMeshArray->Num();
						AllTetEMeshArray->SetNum(ElementsOffset + NumElements);
						AllTetNuMeshArray->SetNum(ElementsOffset + NumElements);
						AllTetAlphaJArray->SetNum(ElementsOffset + NumElements);

						for (uint32 edx = 0; edx < NumElements; ++edx)
						{
							(*AllTetEMeshArray)[edx + ElementsOffset] = TetStiffness[edx];
							(*AllTetNuMeshArray)[edx + ElementsOffset] = TetNu[edx];
							(*AllTetAlphaJArray)[edx + ElementsOffset] = AlphaJMesh[edx];
						}
					}


					if (Property.bEnableCorotatedConstraints)
					{

						int32 InitIndex = Evolution->AddConstraintInitRange(1, true);
						int32 ConstraintIndex = Evolution->AddConstraintRuleRange(1, true);

						if (Property.bDoBlended)
						{
							FBlendedXPBDCorotatedTetrahedralConstraints* BlendedCorotatedConstraint =
								new FBlendedXPBDCorotatedTetrahedralConstraints(
									Evolution->Particles(), Elements, TetStiffness, (FSolverReal).3,/*bRecordMetric = */false, Property.BlendedZeta);

							Evolution->ConstraintInits()[InitIndex] =
								[BlendedCorotatedConstraint](FSolverParticles& InParticles, const FSolverReal Dt)
								{
									BlendedCorotatedConstraint->Init();
								};

							Evolution->ConstraintRules()[ConstraintIndex] =
								[BlendedCorotatedConstraint](FSolverParticles& InParticles, const FSolverReal Dt)
								{
									BlendedCorotatedConstraint->ApplyInParallel(InParticles, Dt);
								};

							BlendedCorotatedConstraints.Add(TUniquePtr<FBlendedXPBDCorotatedTetrahedralConstraints>(BlendedCorotatedConstraint));

						}
						else
						{
							FXPBDCorotatedTetrahedralConstraints* CorotatedConstraint =
								new FXPBDCorotatedTetrahedralConstraints(
									Evolution->Particles(), Elements, TetStiffness, TetNu, MoveTemp(AlphaJMesh), GDeformableXPBDCorotatedParams);

							Evolution->ConstraintInits()[InitIndex] =
								[CorotatedConstraint](FSolverParticles& InParticles, const FSolverReal Dt)
								{
									CorotatedConstraint->Init();
								};

							Evolution->ConstraintRules()[ConstraintIndex] =
								[CorotatedConstraint](FSolverParticles& InParticles, const FSolverReal Dt)
								{
									CorotatedConstraint->ApplyInParallel(InParticles, Dt);
								};

							CorotatedConstraints.Add(TUniquePtr<FXPBDCorotatedTetrahedralConstraints>(CorotatedConstraint));
						}
					}


				}

			}
		}

		Chaos::FRange AllUnconstrainedSurfaceElementsSkinRange(INDEX_NONE, INDEX_NONE), AllUnconstrainedSurfaceElementsCorotatedCodRange(INDEX_NONE, INDEX_NONE);

		if (const TManagedArray<int32>* TriangleMeshIndices = Rest.FindAttribute<int32>("ObjectIndices", "TriangleMesh"))
		{
			if (const TManagedArray<FIntVector>* Indices = Rest.FindAttribute<FIntVector>("Indices", FGeometryCollection::FacesGroup))
			{
				if (const TManagedArray<int32>* FaceStarts = Rest.FindAttribute<int32>("FaceStart", FGeometryCollection::GeometryGroup))
				{
					if (const TManagedArray<int32>* FaceCounts = Rest.FindAttribute<int32>("FaceCount", FGeometryCollection::GeometryGroup))
					{
						if (const TManagedArray<int32>* VertexStarts = Rest.FindAttribute<int32>("VertexStart", FGeometryCollection::GeometryGroup))
						{
							if (const TManagedArray<int32>* VertexCounts = Rest.FindAttribute<int32>("VertexCount", FGeometryCollection::GeometryGroup))
							{
								AllUnconstrainedSurfaceElementsCorotatedCodRange.Start =  AllUnconstrainedSurfaceElementsCorotatedCod->Num();
								AllUnconstrainedSurfaceElementsSkinRange.Start =  AllUnconstrainedSurfaceElementsSkin->Num();
								AllUnconstrainedSurfaceElementsCorotatedCodRange.Count = 0;
								AllUnconstrainedSurfaceElementsSkinRange.Count = 0;
								if (const TManagedArray<bool>* bUseSkinConstraints = Rest.FindAttribute<bool>("SkinConstraints", "TriangleMesh"))
								{
									for (int32 i = 0; i < TriangleMeshIndices->Num(); i++)
									{
										const int32 ObjectIndex = (*TriangleMeshIndices)[i];
										const int32 FaceStartIndex = (*FaceStarts)[ObjectIndex];
										const int32 FaceNum = (*FaceCounts)[ObjectIndex];
										if ((*bUseSkinConstraints)[i])
										{
											int32 SurfaceOffset = AllUnconstrainedSurfaceElementsSkin->Num();
											AllUnconstrainedSurfaceElementsSkin->SetNum(SurfaceOffset + FaceNum);
											for (int32 e = FaceStartIndex; e < FaceStartIndex + FaceNum; e++)
											{
												for (int32 j = 0; j < 3; j++)
												{
													(*AllUnconstrainedSurfaceElementsSkin)[e - FaceStartIndex + SurfaceOffset][j] = (*Indices)[e][j] + Range.Start;
												}
											}
											AllUnconstrainedSurfaceElementsSkinRange.Count += FaceNum;
										}
										else
										{
											int32 SurfaceOffset = AllUnconstrainedSurfaceElementsCorotatedCod->Num();
											AllUnconstrainedSurfaceElementsCorotatedCod->SetNum(SurfaceOffset + FaceNum);
											for (int32 e = FaceStartIndex; e < FaceStartIndex + FaceNum; e++)
											{
												for (int32 j = 0; j < 3; j++)
												{
													(*AllUnconstrainedSurfaceElementsCorotatedCod)[e - FaceStartIndex + SurfaceOffset][j] = (*Indices)[e][j] + Range.Start;
												}
											}
											AllUnconstrainedSurfaceElementsCorotatedCodRange.Count += FaceNum;
										}
									}
								}
								else
								{
									for (int32 i = 0; i < TriangleMeshIndices->Num(); i++)
									{
										const int32 ObjectIndex = (*TriangleMeshIndices)[i];
										const int32 FaceStartIndex = (*FaceStarts)[ObjectIndex];
										const int32 FaceNum = (*FaceCounts)[ObjectIndex];
										int32 SurfaceOffset = AllUnconstrainedSurfaceElementsCorotatedCod->Num();
										AllUnconstrainedSurfaceElementsCorotatedCod->SetNum(SurfaceOffset + FaceNum);
										for (int32 e = FaceStartIndex; e < FaceStartIndex + FaceNum; e++)
										{
											for (int32 j = 0; j < 3; j++)
											{
												(*AllUnconstrainedSurfaceElementsCorotatedCod)[e - FaceStartIndex + SurfaceOffset][j] = (*Indices)[e][j] + Range.Start;
											}
										}
										AllUnconstrainedSurfaceElementsCorotatedCodRange.Count += FaceNum;
									}
								}
							}
						}	
					}
				}
			}
		}
		if (AllCorotatedCodEMeshArray)
		{
			if (AllUnconstrainedSurfaceElementsCorotatedCodRange.Count > 0 )
			{
				AllCorotatedCodEMeshArray->SetNum(AllCorotatedCodEMeshArray->Num() + AllUnconstrainedSurfaceElementsCorotatedCodRange.Count);
				for (int32 i = AllUnconstrainedSurfaceElementsCorotatedCodRange.Start; i < AllUnconstrainedSurfaceElementsCorotatedCodRange.Start + AllUnconstrainedSurfaceElementsCorotatedCodRange.Count; i++)
				{
					(*AllCorotatedCodEMeshArray)[i] = 0.f;
				}
			}
		}
		if (AllSkinEMeshArray)
		{
			if (AllUnconstrainedSurfaceElementsSkinRange.Count > 0)
			{
				AllSkinEMeshArray->SetNum(AllSkinEMeshArray->Num() + AllUnconstrainedSurfaceElementsSkinRange.Count);
				for (int32 i = AllUnconstrainedSurfaceElementsSkinRange.Start; i < AllUnconstrainedSurfaceElementsSkinRange.Start + AllUnconstrainedSurfaceElementsSkinRange.Count; i++)
				{
					(*AllSkinEMeshArray)[i] = 0.f;
				}
			}
		}
		if (StiffnessWithMultiplier.Num() > 0)
		{
			if (AllUnconstrainedSurfaceElementsCorotatedCodRange.Count > 0 && AllCorotatedCodEMeshArray)
			{
				for (int32 i = AllUnconstrainedSurfaceElementsCorotatedCodRange.Start; i < AllUnconstrainedSurfaceElementsCorotatedCodRange.Start + AllUnconstrainedSurfaceElementsCorotatedCodRange.Count; i++)
				{
					(*AllCorotatedCodEMeshArray)[i] = (StiffnessWithMultiplier[(*AllUnconstrainedSurfaceElementsCorotatedCod)[i][0] - Range.Start] + StiffnessWithMultiplier[(*AllUnconstrainedSurfaceElementsCorotatedCod)[i][1] - Range.Start]
									+ StiffnessWithMultiplier[(*AllUnconstrainedSurfaceElementsCorotatedCod)[i][2] - Range.Start]) / 3.f;
				}
			}
			if (AllSkinEMeshArray && AllUnconstrainedSurfaceElementsSkinRange.Count > 0)
			{
				for (int32 i = AllUnconstrainedSurfaceElementsSkinRange.Start; i < AllUnconstrainedSurfaceElementsSkinRange.Start + AllUnconstrainedSurfaceElementsSkinRange.Count; i++)
				{
					(*AllSkinEMeshArray)[i] = (StiffnessWithMultiplier[(*AllUnconstrainedSurfaceElementsSkin)[i][0] - Range.Start] + StiffnessWithMultiplier[(*AllUnconstrainedSurfaceElementsSkin)[i][1] - Range.Start]
									+ StiffnessWithMultiplier[(*AllUnconstrainedSurfaceElementsSkin)[i][2] - Range.Start]) / 3.f;
				}
			}
		}
	}


	void FDeformableSolver::InitializeGridBasedConstraints(FFleshThreadingProxy& Proxy)
	{
		PERF_SCOPE(STAT_ChaosDeformableSolver_InitializeGridBasedConstraints);

		if (Property.bUseGridBasedConstraints)
		{
			auto ChaosTet = [](FIntVector4 V, int32 dp) { return Chaos::TVec4<int32>(dp + V.X, dp + V.Y, dp + V.Z, dp + V.W); };
			const FManagedArrayCollection& Rest = Proxy.GetRestCollection();
			if (const TManagedArray<FIntVector4>* Tetrahedron = Rest.FindAttributeTyped<FIntVector4>("Tetrahedron", "Tetrahedral"))
			{
				if (uint32 NumElements = Tetrahedron->Num())
				{
					const Chaos::FRange& Range = Proxy.GetSolverParticleRange();

					int32 ElementsOffset = AllElements->Num();
					AllElements->SetNum(ElementsOffset + NumElements);
					for (uint32 edx = 0; edx < NumElements; ++edx)
					{
						(*AllElements)[edx + ElementsOffset] = ChaosTet((*Tetrahedron)[edx], Range.Start);
					}
				}
			}
		}
	}

	void FDeformableSolver::InitializeGaussSeidelConstraints(FFleshThreadingProxy& Proxy)
	{
		PERF_SCOPE(STAT_ChaosDeformableSolver_InitializeGaussSeidelConstraint);

		if (Property.bUseGaussSeidelConstraints)
		{
			auto ChaosTet = [](FIntVector4 V, int32 dp) { return Chaos::TVec4<int32>(dp + V.X, dp + V.Y, dp + V.Z, dp + V.W); };
			const FManagedArrayCollection& Rest = Proxy.GetRestCollection();
			const TManagedArray<FIntVector4>* Tetrahedron = Rest.FindAttributeTyped<FIntVector4>("Tetrahedron", "Tetrahedral");
			const TManagedArray<TArray<int32>>* IncidentElementsPointer = Rest.FindAttribute<TArray<int32>>("IncidentElements", "Vertices");
			const TManagedArray<TArray<int32>>* IncidentElementsLocalPointer = Rest.FindAttribute<TArray<int32>>("IncidentElementsLocalIndex", "Vertices");
			if (Tetrahedron && IncidentElementsPointer && IncidentElementsLocalPointer)
			{
				if (uint32 NumElements = Tetrahedron->Num())
				{
					const Chaos::FRange& Range = Proxy.GetSolverParticleRange();

					int32 ElementsOffset = AllElements->Num();
					AllElements->SetNum(ElementsOffset + NumElements);
					for (uint32 edx = 0; edx < NumElements; ++edx)
					{
						(*AllElements)[edx + ElementsOffset] = ChaosTet((*Tetrahedron)[edx], Range.Start);
					}
					if (uint32 NumIncident = IncidentElementsPointer->Num())
					{
						int32 IncidentOffSet = Range.Start;
						AllIncidentElements->SetNum(IncidentOffSet + NumIncident);
						AllIncidentElementsLocal->SetNum(IncidentOffSet + NumIncident);
						for (uint32 i = 0; i < NumIncident; ++i)
						{
							(*AllIncidentElements)[i + IncidentOffSet] = (*IncidentElementsPointer)[i];
							for (int32 j = 0; j < (*AllIncidentElements)[i + IncidentOffSet].Num(); j++)
							{
								(*AllIncidentElements)[i + IncidentOffSet][j] += ElementsOffset;
							}
							(*AllIncidentElementsLocal)[i + IncidentOffSet] = (*IncidentElementsLocalPointer)[i];
						}
					}
				}
			}
		}
	}


	void FDeformableSolver::InitializeKinematicConstraint()
	{
		auto MKinematicUpdate = [this](FSolverParticles& MParticles, const FSolverReal Dt, const FSolverReal MTime, const int32 Index)
		{
			PERF_SCOPE(STAT_ChaosDeformableSolver_InitializeKinematicConstraint);

			if (0 <= Index && Index < this->MObjects.Num())
			{

				if (TransientConstraintBuffer.Contains(Index))
				{
					return;
				}

				if (const UObject* Owner = this->MObjects[Index])
				{
					if (const FFleshThreadingProxy* Proxy = Proxies[Owner]->As<FFleshThreadingProxy>())
					{
						if (!Proxy->GetIsCached())
						{
							FTransform GlobalTransform = Proxy->GetCurrentPointsTransform();
							const Chaos::FRange& Range = Proxy->GetSolverParticleRange();
							const FManagedArrayCollection& Rest = Proxy->GetRestCollection();

							if (Rest.FindAttributeTyped<FVector3f>("Vertex", FGeometryCollection::VerticesGroup))
							{
								const TManagedArray<FVector3f>& Vertex = Rest.GetAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup);
								// @todo(chaos) : reduce conversions
								auto ChaosVert = [](FVector3f V) { return Chaos::FVec3(V.X, V.Y, V.Z); };
								auto ChaosVertfloat = [](FVector3f V) { return Chaos::TVector<FSolverReal, 3>(V.X, V.Y, V.Z); };
								auto SolverParticleToObjectVertexIndex = [&](int32 SolverParticleIndex) {return SolverParticleIndex - Range.Start; };

								FFleshThreadingProxy::FFleshInputBuffer* FleshInputBuffer = nullptr;
								if (this->CurrentInputPackage && this->CurrentInputPackage->ObjectMap.Contains(Owner))
								{
									FleshInputBuffer = this->CurrentInputPackage->ObjectMap[Owner]->As<FFleshThreadingProxy::FFleshInputBuffer>();
								}

								typedef GeometryCollection::Facades::FVertexBoneWeightsFacade FWeightsFacade;
								bool bParticleTouched = false;
								FWeightsFacade WeightsFacade(Rest);
								if (WeightsFacade.HasValidBoneIndicesAndWeights())
								{
									int32 NumObjectVertices = Rest.NumElements(FGeometryCollection::VerticesGroup);
									int32 ObjectVertexIndex = SolverParticleToObjectVertexIndex(Index);
									if (ensure(0 <= ObjectVertexIndex && ObjectVertexIndex < NumObjectVertices))
									{
										if (FleshInputBuffer)
										{
											const TArray<int32>& BoneIndices = WeightsFacade.GetBoneIndices()[ObjectVertexIndex];
											const TArray<float>& BoneWeights = WeightsFacade.GetBoneWeights()[ObjectVertexIndex];

											FFleshThreadingProxy::FFleshInputBuffer* PreviousFleshBuffer = nullptr;
											if (this->PreviousInputPackage && this->PreviousInputPackage->ObjectMap.Contains(Owner))
											{
												PreviousFleshBuffer = this->PreviousInputPackage->ObjectMap[Owner]->As<FFleshThreadingProxy::FFleshInputBuffer>();
											}

											MParticles.SetX(Index, Chaos::TVector<FSolverReal, 3>((FSolverReal)0.));
											TVector<FSolverReal, 3> TargetPos((FSolverReal)0.);
											FSolverReal CurrentRatio = FSolverReal(this->Iteration) / FSolverReal(this->Property.NumSolverSubSteps);

											int32 RestNum = FleshInputBuffer->RestTransforms.Num();
											int32 TransformNum = FleshInputBuffer->Transforms.Num();
											if (RestNum > 0 && TransformNum > 0)
											{

												for (int32 i = 0; i < BoneIndices.Num(); i++)
												{
													if (BoneIndices[i] > INDEX_NONE && BoneIndices[i] < RestNum && BoneIndices[i] < TransformNum)
													{

														// @todo(flesh) : Add the pre-cached component space rest transforms to the rest collection. 
														// see  UFleshComponent::NewDeformableData for how its pulled from the SkeletalMesh
														FVec3 LocalPoint = FleshInputBuffer->RestTransforms[BoneIndices[i]].InverseTransformPosition(ChaosVert(Vertex[Index - Range.Start]));
														FVec3 ComponentPointAtT = FleshInputBuffer->Transforms[BoneIndices[i]].TransformPosition(LocalPoint);

														if (PreviousFleshBuffer)
														{
															FTransform BonePreviousTransform = PreviousFleshBuffer->Transforms[BoneIndices[i]];
															ComponentPointAtT = ComponentPointAtT * CurrentRatio + BonePreviousTransform.TransformPosition(LocalPoint) * ((FSolverReal)1. - CurrentRatio);
														}

														MParticles.SetX(Index, MParticles.GetX(Index) + GlobalTransform.TransformPosition(ComponentPointAtT) * BoneWeights[i]);

														bParticleTouched = true;
													}
												}

											}
											MParticles.PAndInvM(Index).P = MParticles.GetX(Index);
										}
									}
								}
								if (!bParticleTouched && ensure(Vertex.IsValidIndex(Index - Range.Start)))
								{
									MParticles.SetX(Index, GlobalTransform.TransformPosition(ChaosVert(Vertex[Index - Range.Start])));
									MParticles.PAndInvM(Index).P = MParticles.GetX(Index);
								}
							}
						}

#if WITH_EDITOR
						//debug draw
						//p.Chaos.DebugDraw.Enabled 1
						//p.Chaos.DebugDraw.Deformable.KinematicParticle 1
						if (GDeformableDebugParams.IsDebugDrawingEnabled() && GDeformableDebugParams.bDoDrawKinematicParticles )
						{
							auto DoubleVert = [](FVector3f V) { return FVector3d(V.X, V.Y, V.Z); };
							Chaos::FDebugDrawQueue::GetInstance().DrawDebugPoint(DoubleVert(MParticles.GetX(Index)), FColor::Red, false, -1.0f, 0, GDeformableDebugParams.ParticleRadius);
						}
#endif

					}
				}
			}


		};
		Evolution->SetKinematicUpdateFunction(MKinematicUpdate);
	}

	void FDeformableSolver::InitializeSelfCollisionVariables()
	{
		PERF_SCOPE(STAT_ChaosDeformableSolver_InitializeSelfCollisionVariables);
		int32 VertexOffset = 0;
		SurfaceElements->SetNum(0);
		TetmeshSurfaceElements->SetNum(0);
		ParticleComponentIndex->SetNum(0);
		SurfaceCollisionVertices->SetNum(0);
		int ComponentOffset = 0;

		for (FThreadingProxy* InProxy : UninitializedProxys_Internal)
		{
			if (FFleshThreadingProxy* Proxy = InProxy->As<FFleshThreadingProxy>())
			{
				const FManagedArrayCollection& Rest = Proxy->GetRestCollection();
				if (const TManagedArray<FVector3f>* Vertex = Rest.FindAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup))
				{
					if (const TManagedArray<FIntVector>* Indices = Rest.FindAttribute<FIntVector>("Indices", FGeometryCollection::FacesGroup))
					{
						int32 SurfaceOffset = SurfaceElements->Num();
						SurfaceElements->SetNum(SurfaceOffset + Indices->Num());
						TArray<int32> SurfaceVertices;
						for (int32 i = 0; i < Indices->Num(); i++)
						{
							for (int32 j = 0; j < 3; j++)
							{
								(*SurfaceElements)[i + SurfaceOffset][j] = VertexOffset + (*Indices)[i][j];
								SurfaceVertices.Add(VertexOffset + (*Indices)[i][j]);
							}
						}
						TSet<int32> UniqueSurfaceVertices(SurfaceVertices);
						GeometryCollection::Facades::FCollisionFacade CollisionFacade(Rest);
						if (CollisionFacade.IsValid()) //Collision vertices are specified
						{
							for (int32 SurfaceVertexIdx : UniqueSurfaceVertices)
							{
								if (CollisionFacade.IsCollisionEnabled(SurfaceVertexIdx - VertexOffset))
								{
									SurfaceCollisionVertices->Add(SurfaceVertexIdx);
								}
							}
						}
						else
						{
							SurfaceCollisionVertices->Append(UniqueSurfaceVertices.Array());
						}
					}

					VertexOffset += Vertex->Num();

					if (!Property.bDoInComponentSpringCollision || Property.bDoSphereRepulsion) //Component-Component collisions or sphere repulsion
					{
						int32 Offset = ParticleComponentIndex->Num();
						ParticleComponentIndex->SetNum(ParticleComponentIndex->Num() + Vertex->Num());
						for (int32 i = 0; i < Vertex->Num(); i++) {
							(*ParticleComponentIndex)[i + Offset] = ComponentOffset;
						}
						int NewComponentOffset = ComponentOffset;
						GeometryCollection::Facades::FCollectionMeshFacade MeshFacade(Rest);
						TArray<int32> ComponentIndex = MeshFacade.GetGeometryGroupIndexArray();
						for (int32 i = 0; i < ComponentIndex.Num(); i++) {
							if (ComponentIndex[i] < 0)
							{
								(*ParticleComponentIndex)[i + Offset] = ComponentIndex[i]; //Isolated Nodes
							}
							else
							{
								(*ParticleComponentIndex)[i + Offset] = ComponentOffset + ComponentIndex[i];
								NewComponentOffset = NewComponentOffset < (*ParticleComponentIndex)[i + Offset] ? (*ParticleComponentIndex)[i + Offset] : NewComponentOffset;
							}
						}
						ComponentOffset = NewComponentOffset + 1;
					}
				}
			}
		}

		SurfaceTriangleMesh->Init(*SurfaceElements);

		VertexOffset = 0;

		for (FThreadingProxy* InProxy : UninitializedProxys_Internal)
		{
			if (FFleshThreadingProxy* Proxy = InProxy->As<FFleshThreadingProxy>())
			{
				const FManagedArrayCollection& Rest = Proxy->GetRestCollection();
				if (const TManagedArray<FVector3f>* Vertex = Rest.FindAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup))
				{
					if (const TManagedArray<FIntVector>* Indices = Rest.FindAttribute<FIntVector>("Indices", FGeometryCollection::FacesGroup))
					{
						if (const TManagedArray<int32>* TriangleMeshIndices = Rest.FindAttribute<int32>("ObjectIndices", "TriangleMesh"))
						{
							if (const TManagedArray<int32>* FaceStarts = Rest.FindAttribute<int32>("FaceStart", FGeometryCollection::GeometryGroup))
							{
								if (const TManagedArray<int32>* FaceCounts = Rest.FindAttribute<int32>("FaceCount", FGeometryCollection::GeometryGroup))
								{
									TSet<int32> TriMeshObjects;
									for (const int32 ObjectIndex : (*TriangleMeshIndices))
									{
										TriMeshObjects.Add(ObjectIndex);
									}
									for (int32 i = 0; i < FaceStarts->Num(); i++)
									{
										if (!TriMeshObjects.Contains(i))
										{
											const int32 FaceStartIndex = (*FaceStarts)[i];
											const int32 FaceNum = (*FaceCounts)[i];

											int32 SurfaceOffset = TetmeshSurfaceElements->Num();
											TetmeshSurfaceElements->SetNum(SurfaceOffset + FaceNum);
											for (int32 e = FaceStartIndex; e < FaceStartIndex + FaceNum; e++)
											{
												for (int32 j = 0; j < 3; j++)
												{
													(*TetmeshSurfaceElements)[e - FaceStartIndex + SurfaceOffset][j] = (*Indices)[e][j];
												}
											}
										}
									}
								}
							}

						}
						else
						{
							int32 SurfaceOffset = TetmeshSurfaceElements->Num();
							TetmeshSurfaceElements->SetNum(SurfaceOffset + Indices->Num());
							for (int32 i = 0; i < Indices->Num(); i++)
							{
								for (int32 j = 0; j < 3; j++)
								{
									(*TetmeshSurfaceElements)[i + SurfaceOffset][j] = VertexOffset + (*Indices)[i][j];
								}
							}
						}
					}
					VertexOffset += Vertex->Num();
				}
			}
		}
		SurfaceTriangleMesh->Init(*TetmeshSurfaceElements);


		TriangleMeshCollisions.Reset(new FPBDTriangleMeshCollisions(
			0, Evolution->Particles().Size(), *SurfaceTriangleMesh, false, false));
		ParticleTriangleExclusionMap.Reset();
		if (Property.bDoInComponentSpringCollision)
		{
			int32 NRadius = Property.NRingExcluded;
			SurfaceTriangleMesh->GetPointToNeighborsMap();
			SurfaceTriangleMesh->GetPointToTriangleMap(); //Initialize Maps before parallel
			ParticleTriangleExclusionMap.Reserve(SurfaceCollisionVertices->Num());
			PhysicsParallelFor(SurfaceCollisionVertices->Num(),
				[this, &NRadius](int32 i)
				{
					int32 VertexId = (*SurfaceCollisionVertices)[i];
					TSet<int32>& TriangleSet = ParticleTriangleExclusionMap.FindOrAdd(VertexId);
					TSet<int32> NRing;
					if (NRadius > 1)
					{
						NRing = SurfaceTriangleMesh->GetNRing(VertexId, NRadius - 1);
					}
					else if (NRadius == 1)
					{
						NRing.Add(VertexId);
					}
					for (TSet<int32>::TConstIterator It = NRing.CreateConstIterator(); It; ++It)
					{
						TArray<int32> CoincidentTriangles = SurfaceTriangleMesh->GetCoincidentTriangles(*It);
						for (int32 j = 0; j < CoincidentTriangles.Num(); ++j)
						{
							TriangleSet.Add(CoincidentTriangles[j]);
						}
					}
				},true //force single-threaded
			);
		}
	}

	void FDeformableSolver::InitializeGridBasedConstraintVariables()
	{
		PERF_SCOPE(STAT_ChaosDeformableSolver_InitializeGridBasedConstraintVariables);

		GridBasedCorotatedConstraint.Reset(new Chaos::Softs::FXPBDGridBasedCorotatedTetrahedralConstraints(
			Evolution->Particles(), *AllElements, Property.GridDx, /*bRecordMetric = */false, (Chaos::Softs::FSolverReal).1, (Chaos::Softs::FSolverReal).01, (Chaos::Softs::FSolverReal).4, (Chaos::Softs::FSolverReal)1000.0));
		Evolution->ResetConstraintRules();
		int32 InitIndex1 = Evolution->AddConstraintInitRange(1, true);
		Evolution->ConstraintInits()[InitIndex1] =
			[this](FSolverParticles& InParticles, const FSolverReal Dt)
		{
			this->GridBasedCorotatedConstraint->Init(InParticles, Dt);
		};
		int32 ConstraintIndex1 = Evolution->AddConstraintRuleRange(1, true);
		Evolution->ConstraintRules()[ConstraintIndex1] =
			[this](FSolverParticles& InParticles, const FSolverReal Dt)
		{
			this->GridBasedCorotatedConstraint->ApplyInParallel(InParticles, Dt);
		};
		int32 PostprocessingIndex1 = Evolution->AddConstraintPostprocessingsRange(1, true);
		Evolution->ConstraintPostprocessings()[PostprocessingIndex1] =
			[this](FSolverParticles& InParticles, const FSolverReal Dt)
		{
			this->GridBasedCorotatedConstraint->TimeStepPostprocessing(InParticles, Dt);
		};
	}

	void FDeformableSolver::InitializeGaussSeidelConstraintVariables()
	{
		PERF_SCOPE(STAT_ChaosDeformableSolver_InitializeGaussSeidelConstraintVariables);

		GSMainConstraint.Reset(new Chaos::Softs::FGaussSeidelMainConstraints(Evolution->Particles(), Property.bDoQuasistatics, Property.bUseSOR, Property.OmegaSOR, GSParallelMax, MaxDxRatio));

		if (AllUnconstrainedSurfaceElementsCorotatedCod->Num() > 0) 
		{
			GSCorotatedCodConstraints.Reset(new Chaos::Softs::FGaussSeidelCorotatedTriangleConstraints(
				Evolution->Particles(), *AllUnconstrainedSurfaceElementsCorotatedCod, *AllCorotatedCodEMeshArray));
			TArray<TArray<int32>> IncidentElements, IncidentElementsLocal;
			GSMainConstraint->AddStaticConstraints(GSCorotatedCodConstraints->GetConstraintsArray(), IncidentElements, IncidentElementsLocal);

			int32 StaticIndex = GSMainConstraint->AddStaticConstraintResidualAndHessianRange(1);

			GSMainConstraint->StaticConstraintResidualAndHessian()[StaticIndex] = [this](const FSolverParticlesRange& Particles, const int32 ElementIndex, const int32 ElementIndexLocal, const FSolverReal Dt, TVec3<FSolverReal>& ParticleResidual, Chaos::PMatrix<FSolverReal, 3, 3>& ParticleHessian)
			{
				this->GSCorotatedCodConstraints->AddHyperelasticResidualAndHessian(Particles, ElementIndex, ElementIndexLocal, Dt, ParticleResidual, ParticleHessian);
			};
		}

		if (AllUnconstrainedSurfaceElementsSkin->Num() > 0)
		{
			GSLinearCodConstraints.Reset(new Chaos::Softs::FGaussSeidelLinearTriangleConstraints(
				Evolution->Particles(), *AllUnconstrainedSurfaceElementsSkin, *AllSkinEMeshArray));
			TArray<TArray<int32>> IncidentElements, IncidentElementsLocal;
			GSMainConstraint->AddStaticConstraints(GSLinearCodConstraints->GetConstraintsArray(), IncidentElements, IncidentElementsLocal);

			int32 StaticIndex = GSMainConstraint->AddStaticConstraintResidualAndHessianRange(1);

			GSMainConstraint->StaticConstraintResidualAndHessian()[StaticIndex] = [this](const FSolverParticlesRange& Particles, const int32 ElementIndex, const int32 ElementIndexLocal, const FSolverReal Dt, TVec3<FSolverReal>& ParticleResidual, Chaos::PMatrix<FSolverReal, 3, 3>& ParticleHessian)
			{
				this->GSLinearCodConstraints->AddHyperelasticResidualAndHessian(Particles, ElementIndex, ElementIndexLocal, Dt, ParticleResidual, ParticleHessian);
			};
		}

		if (Property.bUseGSNeohookean)
		{
			GSNeohookeanConstraints.Reset(new Chaos::Softs::FGaussSeidelNeohookeanTetrahedralConstraints(
				Evolution->Particles(), *AllElements, *AllTetEMeshArray, *AllTetNuMeshArray, MoveTemp(*AllTetAlphaJArray), MoveTemp(*AllIncidentElements), MoveTemp(*AllIncidentElementsLocal), 0, Evolution->Particles().Size(), Property.bDoQuasistatics, Property.bUseSOR, Property.OmegaSOR, GDeformableXPBDCorotatedParams));
			Evolution->ResetConstraintRules();
			
			GSMainConstraint->AddStaticConstraints(GSNeohookeanConstraints->GetMeshArray(), GSNeohookeanConstraints->GetIncidentElements(), GSNeohookeanConstraints->GetIncidentElementsLocal());

			int32 InitIndex1 = Evolution->AddConstraintInitRange(1, true);
			Evolution->ConstraintInits()[InitIndex1] =
				[this](FSolverParticles& InParticles, const FSolverReal Dt)
			{
				this->GSMainConstraint->Init(Dt, InParticles);
			};

			int32 ConstraintIndex1 = Evolution->AddConstraintRuleRange(1, true);
			Evolution->ConstraintRules()[ConstraintIndex1] =
				[this](FSolverParticles& InParticles, const FSolverReal Dt)
			{
				this->GSMainConstraint->Apply(InParticles, Dt, 10, false, &(this->Evolution->ParticlesActiveView()));
			};


			int32 StaticIndex = GSMainConstraint->AddStaticConstraintResidualAndHessianRange(1);

			GSMainConstraint->StaticConstraintResidualAndHessian()[StaticIndex] = [this](const FSolverParticlesRange& Particles, const int32 ElementIndex, const int32 ElementIndexLocal, const FSolverReal Dt, TVec3<FSolverReal>& ParticleResidual, Chaos::PMatrix<FSolverReal, 3, 3>& ParticleHessian)
			{
				this->GSNeohookeanConstraints->AddHyperelasticResidualAndHessian(Particles, ElementIndex, ElementIndexLocal, Dt, ParticleResidual, ParticleHessian);
			};
		} 
		else
		{
			GSCorotatedConstraints.Reset(new Chaos::Softs::FGaussSeidelCorotatedTetrahedralConstraints(
				Evolution->Particles(), *AllElements, *AllTetEMeshArray, *AllTetNuMeshArray, MoveTemp(*AllTetAlphaJArray), MoveTemp(*AllIncidentElements), MoveTemp(*AllIncidentElementsLocal), 0, Evolution->Particles().Size(), Property.bDoQuasistatics, Property.bUseSOR, Property.OmegaSOR, GDeformableXPBDCorotatedParams));
			Evolution->ResetConstraintRules();

			GSMainConstraint->AddStaticConstraints(GSCorotatedConstraints->GetMeshArray(), GSCorotatedConstraints->GetIncidentElements(), GSCorotatedConstraints->GetIncidentElementsLocal());
			
			int32 InitIndex1 = Evolution->AddConstraintInitRange(1, true);
			Evolution->ConstraintInits()[InitIndex1] =
				[this](FSolverParticles& InParticles, const FSolverReal Dt)
			{
				this->GSMainConstraint->Init(Dt, InParticles);
			};

			int32 ConstraintIndex1 = Evolution->AddConstraintRuleRange(1, true);
			Evolution->ConstraintRules()[ConstraintIndex1] =
				[this](FSolverParticles& InParticles, const FSolverReal Dt)
				{
					this->GSMainConstraint->Apply(InParticles, Dt, 10, false, &(this->Evolution->ParticlesActiveView()));
				};


			int32 StaticIndex = GSMainConstraint->AddStaticConstraintResidualAndHessianRange(1);

			GSMainConstraint->StaticConstraintResidualAndHessian()[StaticIndex] = [this](const FSolverParticlesRange& Particles, const int32 ElementIndex, const int32 ElementIndexLocal, const FSolverReal Dt, TVec3<FSolverReal>& ParticleResidual, Chaos::PMatrix<FSolverReal, 3, 3>& ParticleHessian)
				{
					this->GSCorotatedConstraints->AddHyperelasticResidualAndHessian(Particles, ElementIndex, ElementIndexLocal, Dt, ParticleResidual, ParticleHessian);
				};
		}

		if (Property.bEnablePositionTargets)
		{
			GSWeakConstraints->ComputeInitialWCData(Evolution->Particles());

			const TArray<TArray<int32>>* StaticIncidentElements = nullptr;
			const TArray<TArray<int32>>* StaticIncidentElementsLocal = nullptr;
			const TArray<TArray<int32>>& StaticConstraints = GSWeakConstraints->GetStaticConstraintArrays(StaticIncidentElements, StaticIncidentElementsLocal);
			GSMainConstraint->AddStaticConstraints(StaticConstraints, *StaticIncidentElements, *StaticIncidentElementsLocal);

			const int32 StaticIndex1 = GSMainConstraint->AddStaticConstraintResidualAndHessianRange(1);
			GSMainConstraint->StaticConstraintResidualAndHessian()[StaticIndex1] = [this](const FSolverParticlesRange& Particles, const int32 ConstraintIndex, const int32 ConstraintIndexLocal, const FSolverReal Dt, TVec3<FSolverReal>& ParticleResidual, Chaos::PMatrix<FSolverReal, 3, 3>& ParticleHessian)
				{
					this->GSWeakConstraints->AddWCResidual(Particles, ConstraintIndex, ConstraintIndexLocal, Dt, ParticleResidual, ParticleHessian);
				};

			int32 PerNodeIndex = GSMainConstraint->AddPerNodeHessianRange(1);
			GSMainConstraint->PerNodeHessian()[PerNodeIndex] = [this](const int32 p, const FSolverReal Dt, Chaos::PMatrix<FSolverReal, 3, 3>& ParticleHessian)
				{
					this->GSWeakConstraints->AddWCHessian(p, Dt, ParticleHessian);
				};
		}

		if (GSVolumeConstraints->NumConstraints())
		{
			const TArray<TArray<int32>>* StaticIncidentElements = nullptr;
			const TArray<TArray<int32>>* StaticIncidentElementsLocal = nullptr;
			const TArray<TArray<int32>>& StaticConstraints = GSVolumeConstraints->GetStaticConstraintArrays(StaticIncidentElements, StaticIncidentElementsLocal);
			GSMainConstraint->AddStaticConstraints(StaticConstraints, *StaticIncidentElements, *StaticIncidentElementsLocal);

			const int32 StaticIndex1 = GSMainConstraint->AddStaticConstraintResidualAndHessianRange(1);
			GSMainConstraint->StaticConstraintResidualAndHessian()[StaticIndex1] = [this](const FSolverParticlesRange& Particles, const int32 ConstraintIndex, const int32 ConstraintIndexLocal, const FSolverReal Dt, TVec3<FSolverReal>& ParticleResidual, Chaos::PMatrix<FSolverReal, 3, 3>& ParticleHessian)
				{
					this->GSVolumeConstraints->AddResidualAndHessian(
						Evolution->Particles(),
						ConstraintIndex, ConstraintIndexLocal, Dt, ParticleResidual, ParticleHessian);
				};
		}

		GSMainConstraint->InitStaticColor(Evolution->Particles(), &(Evolution->ParticlesActiveView()));

		if (Property.bEnablePositionTargets)
		{
			int32 InitIndex1 = Evolution->AddConstraintInitRange(1, true);
			Evolution->ConstraintInits()[InitIndex1] =
				[this](FSolverParticles& InParticles, const FSolverReal Dt)
				{
					this->GSWeakConstraints->Init(InParticles, Dt);
				};
		}

		if (Property.bEnableDynamicSprings)
		{
			int32 DynamicIndex = GSMainConstraint->AddDynamicConstraintResidualAndHessianRange(1);
			GSMainConstraint->DynamicConstraintResidualAndHessian()[DynamicIndex] = [this](const FSolverParticlesRange& Particles, const int32 ConstraintIndex, const int32 ConstraintIndexLocal, const FSolverReal Dt, TVec3<FSolverReal>& ParticleResidual, Chaos::PMatrix<FSolverReal, 3, 3>& ParticleHessian)
			{
				this->GSDynamicWeakConstraints->AddWCResidual(Particles, ConstraintIndex, ConstraintIndexLocal, Dt, ParticleResidual, ParticleHessian);
			};

			int32 PerNodeIndex = GSMainConstraint->AddPerNodeHessianRange(1);
			GSMainConstraint->PerNodeHessian()[PerNodeIndex] = [this](const int32 p, const FSolverReal Dt, Chaos::PMatrix<FSolverReal, 3, 3>& ParticleHessian)
			{
				this->GSDynamicWeakConstraints->AddWCHessian(p, Dt, ParticleHessian);
			};

			int32 InitIndex1 = Evolution->AddConstraintInitRange(1, true);

			GSDynamicWeakConstraints->ComputeInitialWCData(Evolution->Particles());

			Evolution->ConstraintInits()[InitIndex1] =
				[this](FSolverParticles& InParticles, const FSolverReal Dt)
				{
					this->GSDynamicWeakConstraints->Init(InParticles, Dt);
					if (this->bDynamicConstraintIsUpdated)
					{
						const TArray<TArray<int32>>* WCDynamicIncidentElements = nullptr;
						const TArray<TArray<int32>>* WCDynamicIncidentElementsLocal = nullptr;
						const TArray<TArray<int32>>& DynamicConstraints = GSDynamicWeakConstraints->GetStaticConstraintArrays(WCDynamicIncidentElements, WCDynamicIncidentElementsLocal);
						GSMainConstraint->ResetDynamicConstraints();
						GSMainConstraint->AddDynamicConstraints(DynamicConstraints, *WCDynamicIncidentElements, *WCDynamicIncidentElementsLocal, true);
						this->GSMainConstraint->InitDynamicColor(InParticles);
					}
				};
		}

		if (Property.bDoSpringCollision)
		{
			int32 TransientIndex = GSMainConstraint->AddTransientConstraintResidualAndHessianRange(1);
			GSMainConstraint->TransientConstraintResidualAndHessian()[TransientIndex] = [this](const FSolverParticlesRange& Particles, const int32 ConstraintIndex, const int32 ConstraintIndexLocal, const FSolverReal Dt, TVec3<FSolverReal>& ParticleResidual, Chaos::PMatrix<FSolverReal, 3, 3>& ParticleHessian)
				{
					this->GSWeakConstraints->AddWCResidual(Particles, ConstraintIndex + this->GSWeakConstraints->GetInitialWCSize(), ConstraintIndexLocal, Dt, ParticleResidual, ParticleHessian);
				};

			if (!Property.bEnablePositionTargets)
			{
				int32 PerNodeIndex = GSMainConstraint->AddPerNodeHessianRange(1);
				GSMainConstraint->PerNodeHessian()[PerNodeIndex] = [this](const int32 p, const FSolverReal Dt, Chaos::PMatrix<FSolverReal, 3, 3>& ParticleHessian)
					{
						this->GSWeakConstraints->AddWCHessian(p, Dt, ParticleHessian);
					};
			}

			int32 InitIndex = Evolution->AddConstraintInitRange(1, true);
			Evolution->ConstraintInits()[InitIndex] =
				[this](FSolverParticles& InParticles, const FSolverReal Dt)
				{
					this->TriangleMeshCollisions->InitFlesh(InParticles, Property.SpringCollisionSearchRadius, this->Property.bCollideWithFullMesh);
					if (Property.bDoInComponentSpringCollision)
					{
						this->GSWeakConstraints->CollisionDetectionSpatialHashInComponent(this->Evolution->Particles(), *SurfaceCollisionVertices, *SurfaceTriangleMesh, ParticleTriangleExclusionMap, TriangleMeshCollisions->GetDynamicSpatialHash(), Property.SpringCollisionSearchRadius, Property.SpringCollisionStiffness, Property.bAllowSliding);
					}
					else
					{
						this->GSWeakConstraints->CollisionDetectionSpatialHash(this->Evolution->Particles(), *SurfaceCollisionVertices, *SurfaceTriangleMesh, *ParticleComponentIndex, TriangleMeshCollisions->GetDynamicSpatialHash(), Property.SpringCollisionSearchRadius, Property.SpringCollisionStiffness, Property.bAllowSliding);
					}
					TArray<TArray<int32>> WCCollisionConstraints, WCCollisionIncidentElements, WCCollisionIncidentElementsLocal;
					this->GSWeakConstraints->ComputeCollisionWCDataSimplified(WCCollisionConstraints, WCCollisionIncidentElements, WCCollisionIncidentElementsLocal);
					this->GSMainConstraint->AddTransientConstraints(WCCollisionConstraints, WCCollisionIncidentElements, WCCollisionIncidentElementsLocal);
					this->GSMainConstraint->InitTransientColor(InParticles);
				};
		}
		if (Property.bDoSphereRepulsion)
		{
			GSSphereRepulsionConstraints.Reset(new FGaussSeidelParticleSphereRepulsionConstraints(Property.SphereRepulsionRadius, Property.SphereRepulsionStiffness, Evolution->Particles(), GDeformableXPBDWeakConstraintParams));
			int32 InitIndex = Evolution->AddConstraintInitRange(1, true);
			
			Evolution->ConstraintInits()[InitIndex] 
				= [this](FSolverParticles& InParticles, const FSolverReal Dt)
				{
					this->GSSphereRepulsionConstraints->UpdateSphereRepulsionConstraints(this->Evolution->Particles(), *SurfaceCollisionVertices, *ParticleComponentIndex);
					TArray<TArray<int32>> SphereRepulsionConstraints, SphereRepulsionIncidentElements, SphereRepulsionIncidentElementsLocal;
					this->GSSphereRepulsionConstraints->ReturnSphereRepulsionConstraints(SphereRepulsionConstraints, SphereRepulsionIncidentElements, SphereRepulsionIncidentElementsLocal);
					this->GSMainConstraint->AddTransientConstraints(SphereRepulsionConstraints, SphereRepulsionIncidentElements, SphereRepulsionIncidentElementsLocal);
					this->GSMainConstraint->InitTransientColor(InParticles);
					this->GSSphereRepulsionConstraints->Init(InParticles, Dt);
				};

			int32 TransientIndex = GSMainConstraint->AddTransientConstraintResidualAndHessianRange(1);
			GSMainConstraint->TransientConstraintResidualAndHessian()[TransientIndex] =
				[this](const FSolverParticlesRange& Particles, const int32 ConstraintIndex, const int32 ConstraintIndexLocal, const FSolverReal Dt, TVec3<FSolverReal>& ParticleResidual, Chaos::PMatrix<FSolverReal, 3, 3>& ParticleHessian)
				{
					this->GSSphereRepulsionConstraints->AddSphereRepulsionResidualAndHessian(Particles, ConstraintIndex, ConstraintIndexLocal, Dt, ParticleResidual, ParticleHessian);
				};
		}
		if (Property.bEnableGravity && Property.bDoQuasistatics) // Quasistatic PBD evolution does not apply gravity, add here
		{
			// @todo(flesh): Each proxy may have different sim spaces and therefore different gravity directions
			const uint32 GroupId = 0;
			FSolverVec3 GravityDir = Evolution->GetGravity(GroupId);
			GSMainConstraint->AddExternalAcceleration(GravityDir);
		}
	}

	void FDeformableSolver::InitializeMuscleActivationVariables()
	{
		// Apply muscle activation lambda
		int32 InitIndex = Evolution->AddConstraintInitRange(1, true);
		Evolution->ConstraintInits()[InitIndex] = [this](FSolverParticles& InParticles, const FSolverReal Dt)
		{
			if (Property.bDoLengthBasedMuscleActivation)
			{
				this->MuscleActivationConstraints->UpdateLengthBasedMuscleActivation(InParticles);
			}
			if (Property.bOverrideMuscleActivationWithAnimatedCurves)
			{
				//Override muscle activation with curve data
				for (const TPair<FThreadingProxy::FKey, TUniquePtr<FThreadingProxy>>& ProxyElem : Proxies)
				{
					if (const FFleshThreadingProxy* Proxy = ProxyElem.Value->As<FFleshThreadingProxy>())
					{
						if (const UObject* Owner = Proxy->GetOwner())
						{
							if (!Proxy->GetIsCached() && MuscleIndexOffset.Contains(Owner))
							{
								if (this->CurrentInputPackage && this->CurrentInputPackage->ObjectMap.Contains(Owner))
								{
									if (FFleshThreadingProxy::FFleshInputBuffer* FleshInputBuffer = this->CurrentInputPackage->ObjectMap[Owner]->As<FFleshThreadingProxy::FFleshInputBuffer>())
									{
										const TArray<int32>& MuscleIndices = FleshInputBuffer->MuscleIndices;
										const TArray<float>& MuscleActivations = FleshInputBuffer->MuscleActivation;
										if (ensure(MuscleIndices.Num() == MuscleActivations.Num()))
										{
											const int32 Offset = MuscleIndexOffset[Owner];
											for (int32 Idx = 0; Idx < MuscleIndices.Num(); ++Idx)
											{
												this->MuscleActivationConstraints->SetMuscleActivation(Offset + MuscleIndices[Idx], MuscleActivations[Idx]);
											}
											if (this->PreviousInputPackage && this->PreviousInputPackage->ObjectMap.Contains(Owner))
											{
												if (FFleshThreadingProxy::FFleshInputBuffer* PreviousFleshBuffer = this->PreviousInputPackage->ObjectMap[Owner]->As<FFleshThreadingProxy::FFleshInputBuffer>())
												{
													FSolverReal CurrentRatio = FSolverReal(this->Iteration) / FSolverReal(this->Property.NumSolverSubSteps);
													const TArray<int32>& PrevMuscleIndices = PreviousFleshBuffer->MuscleIndices;
													const TArray<float>& PrevMuscleActivations = PreviousFleshBuffer->MuscleActivation;
													for (int32 Idx = 0; Idx < PrevMuscleIndices.Num(); ++Idx)
													{
														const int32 MuscleGlobalIdx = Offset + PrevMuscleIndices[Idx];
														const float CurrentMuscleActivation = this->MuscleActivationConstraints->GetMuscleActivation(MuscleGlobalIdx);
														this->MuscleActivationConstraints->SetMuscleActivation(MuscleGlobalIdx,
															CurrentRatio * CurrentMuscleActivation + (1 - CurrentRatio) * PrevMuscleActivations[Idx]);
													}
												}
											}
										}
									}
								}
							}
						}
					}
				}
			}

			if (Property.bUseGSNeohookean)
			{
				this->MuscleActivationConstraints->ApplyMuscleActivation(*this->GSNeohookeanConstraints);
			}
			else
			{
				this->MuscleActivationConstraints->ApplyMuscleActivation(*this->GSCorotatedConstraints);
			}
		};

		//Adjust muscle rest volume
		if (MuscleActivationConstraints && MuscleActivationConstraints->NumMuscles())
		{
			if (Property.bUseGSNeohookean)
			{
				MuscleActivationConstraints->ApplyInflationVolumeScale(*GSNeohookeanConstraints);
			}
			else
			{
				MuscleActivationConstraints->ApplyInflationVolumeScale(*GSCorotatedConstraints);
			}
		}
	}

	void FDeformableSolver::InitializeMuscleActivation(FFleshThreadingProxy& Proxy)
	{
		PERF_SCOPE(STAT_ChaosDeformableSolver_InitializeMuscleActivation);
		//Grab MuscleActivationElements, OriginInsertion, FiberDirectionMatrix
		const FManagedArrayCollection& Rest = Proxy.GetRestCollection();
		GeometryCollection::Facades::FMuscleActivationFacade MuscleActivationFacade(Rest);
		if (MuscleActivationFacade.IsValid())
		{
			if (AllElements)
			{
				int32 VertexOffset = Proxy.GetSolverParticleRange().Start;
				// Only supports Gauss Seidel for now because we are using AllElements
				int32 ElementOffset = AllElements->Num(); //InitializeMuscleActivation has to go before adding tetrahedrons
				MuscleIndexOffset.Add(Proxy.GetOwner(), MuscleActivationConstraints->NumMuscles());
				MuscleActivationConstraints->AddMuscles(Evolution->Particles(), MuscleActivationFacade, VertexOffset, ElementOffset);
			}
		}
	}

	void FDeformableSolver::ResetGaussSeidelVariables()
	{
		AllElements.Reset(new TArray<Chaos::TVec4<int32>>());
		AllIncidentElements.Reset(new TArray<TArray<int32>>());
		AllIncidentElementsLocal.Reset(new TArray<TArray<int32>>());
		AllTetEMeshArray.Reset(new TArray<FSolverReal>());
		AllTetNuMeshArray.Reset(new TArray<FSolverReal>());
		AllTetAlphaJArray.Reset(new TArray<FSolverReal>());
		AllCorotatedCodEMeshArray.Reset(new TArray<FSolverReal>());
		AllSkinEMeshArray.Reset(new TArray<FSolverReal>());	
		MuscleIndexOffset.Empty();
		AllUnconstrainedSurfaceElementsCorotatedCod.Reset(new TArray<Chaos::TVec3<int32>>());
		AllUnconstrainedSurfaceElementsSkin.Reset(new TArray<Chaos::TVec3<int32>>());
		GSWeakConstraints.Reset(new FGaussSeidelSpringConstraints({}, {}, {}, {}, {}, {}, {}, GDeformableXPBDWeakConstraintParams));
		GSDynamicWeakConstraints.Reset(new FGaussSeidelDynamicSpringConstraints(GDeformableXPBDWeakConstraintParams));
		GSSphereRepulsionConstraints.Reset(new FGaussSeidelParticleSphereRepulsionConstraints(Property.SphereRepulsionRadius, Property.SphereRepulsionStiffness, Evolution->Particles(), GDeformableXPBDWeakConstraintParams));
		GSVolumeConstraints.Reset(new FGaussSeidelUnilateralVolumeConstraints(Evolution->Particles(), {}, {}));
		MuscleActivationConstraints.Reset(new FMuscleActivationTetrahedralConstraints());
	}

	void FDeformableSolver::RemoveSimulationObjects()
	{
		PERF_SCOPE(STAT_ChaosDeformableSolver_RemoveSimulationObjects);

		TArray< FThreadingProxy* > RemovedProxies;
		{
			FScopeLock Lock(&RemovalMutex); // @todo(flesh) : change to threaded task based commands to prevent the lock. 
			RemovedProxies = TArray< FThreadingProxy* >(RemovedProxys_Internal);
			RemovedProxys_Internal.Empty();
		}

		if (RemovedProxies.Num())
		{
			Evolution->ResetConstraintRules();
			Evolution->DeactivateParticleRanges();

			// Sorting of the proxies to be removed to avoid particles indices issues in the next loop
			RemovedProxies.Sort([](FThreadingProxy& BaseProxyA, FThreadingProxy& BaseProxyB) ->bool
			{
				const FFleshThreadingProxy* FleshProxyA = BaseProxyA.As<FFleshThreadingProxy>();
				const FFleshThreadingProxy* FleshProxyB = BaseProxyB.As<FFleshThreadingProxy>();

				if(FleshProxyA && FleshProxyB)
				{
					return FleshProxyA->GetSolverParticleRange().Start < FleshProxyB->GetSolverParticleRange().Start;
				}
				return true;
			});

			// delete the simulated particles in block moves
			for (FThreadingProxy* BaseProxy : RemovedProxies)
			{
				if (FFleshThreadingProxy* Proxy = BaseProxy->As<FFleshThreadingProxy>())
				{
					if (Proxy->CanSimulate())
					{
						Chaos::FRange Indices = Proxy->GetSolverParticleRange();
						if (Indices.Count > 0)
						{
							Proxies.FindAndRemoveChecked(MObjects[Indices.Start]);
							Evolution->Particles().RemoveAt(Indices.Start, Indices.Count);
						}
					}
				}
			}

			// reindex ranges on moved particles in the proxies. 
			const UObject* CurrentObject = nullptr;
			for (int Index = 0; Index < MObjects.Num(); Index++)
			{
				if (MObjects[Index] != CurrentObject)
				{
					CurrentObject = MObjects[Index];
					if (CurrentObject)
					{
						if (ensure(Proxies.Contains(CurrentObject)))
						{
							if (FFleshThreadingProxy* MovedProxy = Proxies[CurrentObject]->As<FFleshThreadingProxy>())
							{
								Chaos::FRange Range = MovedProxy->GetSolverParticleRange();
								MovedProxy->SetSolverParticleRange(Range.Start, Range.Count);
								int32 Offset = Evolution->AddParticleRange(Range.Count);
								//ensure(Offset == Range.Start);
							}
						}
					}
				}
			}

			ResetGaussSeidelVariables();

			// regenerate all constraints
			for (TPair< FThreadingProxy::FKey, TUniquePtr<FThreadingProxy> >& BaseProxyPair : Proxies)
			{
				if (FFleshThreadingProxy* Proxy = BaseProxyPair.Value->As<FFleshThreadingProxy>())
				{
					InitializeWeakConstraint(*Proxy);
					InitializeMuscleActivation(*Proxy);
					InitializeTetrahedralOrTriangleConstraint(*Proxy);
					InitializeGridBasedConstraints(*Proxy);
					InitializeGaussSeidelConstraints(*Proxy);
				}
			}
		}
	}

	void FDeformableSolver::AdvanceDt(FSolverReal DeltaTime)
	{
		PERF_SCOPE(STAT_ChaosDeformableSolver_AdvanceDt);

		EventPreSolve.Broadcast(DeltaTime);

		const TArray<TVector<int32, 2>, TInlineAllocator<8>>& EvolutionActiveRange = Evolution->ParticlesActiveView().GetActiveRanges();
		bool bActiveRangeAreSame = true;
		if (EvolutionActiveRange.Num() == PrevEvolutionActiveRange.Num())
		{
			for (int32 i = 0; i < PrevEvolutionActiveRange.Num(); i++)
			{
				if (PrevEvolutionActiveRange[i]!= EvolutionActiveRange[i])
				{
					bActiveRangeAreSame = false;
					break;
				}
			}
		}
		else
		{
			bActiveRangeAreSame = false;
		}

		if (!bActiveRangeAreSame)
		{
			if (GSMainConstraint)
			{
				GSMainConstraint->InitStaticColor(Evolution->Particles(), &(Evolution->ParticlesActiveView()));
			}
		}


		int32 NumSubsteps = FMath::Clamp<int32>(Property.NumSolverSubSteps, 0, INT_MAX);
		if (bEnableSolver && NumSubsteps)
		{
			FSolverReal SubDeltaTime = DeltaTime / (FSolverReal)NumSubsteps;
			if (!FMath::IsNearlyZero(SubDeltaTime))
			{
				for (int i = 0; i < NumSubsteps; ++i)
				{
					Iteration = i+1;
					Update(SubDeltaTime);
				}
				PostProcessTransientConstraints();

				Frame++;
				EventPostSolve.Broadcast(DeltaTime);
			}
		}



		{
			// Update client state
			FDeformableDataMap OutputBuffers;
			for (TPair< FThreadingProxy::FKey, TUniquePtr<FThreadingProxy> >& BaseProxyPair : Proxies)
			{
				UpdateOutputState(*BaseProxyPair.Value);
				if (FFleshThreadingProxy* Proxy = BaseProxyPair.Value->As<FFleshThreadingProxy>())
				{
					OutputBuffers.Add(Proxy->GetOwner(), TSharedPtr<FThreadingProxy::FBuffer>(new FFleshThreadingProxy::FFleshOutputBuffer(*Proxy)));

					if (Property.CacheToFile)
					{
						WriteFrame(*Proxy, DeltaTime);
					}
				}
			}
			PushOutputPackage(Frame, MoveTemp(OutputBuffers));
		}

		{
#if WITH_EDITOR
			// debug draw
	
			//p.Chaos.DebugDraw.Enabled 1
			if (GDeformableDebugParams.IsDebugDrawingEnabled())
			{
				for (TPair< FThreadingProxy::FKey, TUniquePtr<FThreadingProxy> >& BaseProxyPair : Proxies)
				{
					if (FFleshThreadingProxy* Proxy = BaseProxyPair.Value->As<FFleshThreadingProxy>())
					{
						if (GDeformableDebugParams.bDoDrawTetrahedralParticles)
						{
							//p.Chaos.DebugDraw.Deformable.TetrahedralParticles 1
							DebugDrawTetrahedralParticles(*Proxy);
						}
					}
				}
			}
#endif
		}

		EventPreBuffer.Broadcast(DeltaTime);
	}

	void FDeformableSolver::PushInputPackage(int32 InFrame, FDeformableDataMap&& InPackage)
	{
		PERF_SCOPE(STAT_ChaosDeformableSolver_PushInputPackage);

		FScopeLock Lock(&PackageInputMutex);
		TRACE_CPUPROFILER_EVENT_SCOPE(ChaosDeformableSolver_PushInputPackage);
		BufferedInputPackages.Push(TUniquePtr< FDeformablePackage >(new FDeformablePackage(InFrame, MoveTemp(InPackage))));
	}

	TUniquePtr<FDeformablePackage> FDeformableSolver::PullInputPackage()
	{
		PERF_SCOPE(STAT_ChaosDeformableSolver_PullInputPackage);

		FScopeLock Lock(&PackageInputMutex);
		TRACE_CPUPROFILER_EVENT_SCOPE(ChaosDeformableSolver_PullInputPackage);
		if (BufferedInputPackages.Num())
			return BufferedInputPackages.Pop();
		return TUniquePtr<FDeformablePackage>(nullptr);
	}

	void FDeformableSolver::PushRestartPackage(int32 InFrame, FDeformableDataMap&& InPackage)
	{
		PERF_SCOPE(STAT_ChaosDeformableSolver_PushRestartPackage);
		FScopeLock Lock(&PackageRestartMutex);
		TRACE_CPUPROFILER_EVENT_SCOPE(ChaosDeformableSolver_PushRestartPackage);
		BufferedRestartPackages.Push(TUniquePtr< FDeformablePackage >(new FDeformablePackage(InFrame, MoveTemp(InPackage))));
		bPendingRestart = true;
	}

	TUniquePtr<FDeformablePackage> FDeformableSolver::PullRestartPackage()
	{
		PERF_SCOPE(STAT_ChaosDeformableSolver_PullRestartPackage);
		FScopeLock Lock(&PackageRestartMutex);
		TRACE_CPUPROFILER_EVENT_SCOPE(ChaosDeformableSolver_PullRestartPackage);
		if (BufferedRestartPackages.Num())
		{
			return BufferedRestartPackages.Pop();
		}
		return {};
	}

	void FDeformableSolver::UpdateProxyInputPackages()
	{
		PERF_SCOPE(STAT_ChaosDeformableSolver_UpdateProxyInputPackages);

		if (CurrentInputPackage)
		{
			PreviousInputPackage = TUniquePtr < FDeformablePackage >(CurrentInputPackage.Release());
			CurrentInputPackage = TUniquePtr < FDeformablePackage >(nullptr);
		}

		TUniquePtr < FDeformablePackage > TailPackage = PullInputPackage();
		while (TailPackage)
		{
			CurrentInputPackage = TUniquePtr < FDeformablePackage >(TailPackage.Release());
			TailPackage = PullInputPackage();
		}
	}

	void FDeformableSolver::Update(FSolverReal DeltaTime)
	{
		PERF_SCOPE(STAT_ChaosDeformableSolver_Update);

		bDynamicConstraintIsUpdated = false;

		if (!Proxies.Num()) return;

		UpdateSimulationObjects(DeltaTime);

		if (!Property.FixTimeStep)
		{
			Evolution->AdvanceOneTimeStep(DeltaTime);
			Time += DeltaTime;
		}
		else
		{
			Evolution->AdvanceOneTimeStep(Property.TimeStepSize);
			Time += Property.TimeStepSize;
		}

	}

	void FDeformableSolver::PushOutputPackage(int32 InFrame, FDeformableDataMap&& InPackage)
	{
		FScopeLock Lock(&PackageOutputMutex);
		PERF_SCOPE(STAT_ChaosDeformableSolver_PushOutputPackage);
		BufferedOutputPackages.Push(TUniquePtr< FDeformablePackage >(new FDeformablePackage(InFrame, MoveTemp(InPackage))));
	}

	TUniquePtr<FDeformablePackage> FDeformableSolver::PullOutputPackage()
	{
		FScopeLock Lock(&PackageOutputMutex);
		PERF_SCOPE(STAT_ChaosDeformableSolver_PullOutputPackage);
		if (BufferedOutputPackages.Num())
			return BufferedOutputPackages.Pop();
		return TUniquePtr<FDeformablePackage>(nullptr);
	}

	void FDeformableSolver::AddProxy(FThreadingProxy* InProxy)
	{
		PERF_SCOPE(STAT_ChaosDeformableSolver_AddProxy);

		FScopeLock Lock(&InitializationMutex);
		UninitializedProxys_Internal.Add(InProxy);
		InitializedObjects_External.Add(InProxy->GetOwner());
	}

	void FDeformableSolver::RemoveProxy(FThreadingProxy* InProxy)
	{
		PERF_SCOPE(STAT_ChaosDeformableSolver_RemoveProxy);

		FScopeLock LockA(&RemovalMutex);
		FScopeLock LockB(&InitializationMutex);

		InitializedObjects_External.Remove(InProxy->GetOwner());

		// If a proxy has not been initialized yet, then we need
		// to clean up the internal buffers. 
		int32 Index = UninitializedProxys_Internal.IndexOfByKey(InProxy);
		if(Index!=INDEX_NONE)
		{
			UninitializedProxys_Internal.RemoveAtSwap(Index);
			if (Proxies.Contains(InProxy->GetOwner()))
			{
				RemovedProxys_Internal.Add(InProxy);
			}
			else
			{
				delete InProxy;
			}
		}
		else if(Proxies.Contains(InProxy->GetOwner()))
		{
			RemovedProxys_Internal.Add(InProxy);
		}

	}

	void FDeformableSolver::UpdateOutputState(FThreadingProxy& InProxy)
	{
		PERF_SCOPE(STAT_ChaosDeformableSolver_UpdateOutputState);

		if (FFleshThreadingProxy* Proxy = InProxy.As<FFleshThreadingProxy>())
		{
			const Chaos::FRange& Range = Proxy->GetSolverParticleRange();
			if (0 <= Range.Start)
			{
				// @todo(chaos) : reduce conversions
				auto UEVertd = [](Chaos::FVec3 V) { return FVector3d(V.X, V.Y, V.Z); };
				auto UEVertf = [](FVector3d V) { return FVector3f((float)V.X, (float)V.Y, (float)V.Z); };

				TManagedArray<FVector3f>& Position = Proxy->GetDynamicCollection().ModifyAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup);

				if((Position.Num() + Range.Start) <= (int32)(Evolution->Particles().Size()))
				{
					// The final transform gets us from whatever the simulation space is,
					// to component space.
					const FTransform FinalXf = Proxy->GetFinalTransform();
					if (!FinalXf.Equals(FTransform::Identity))
					{
						for (int32 vdx = 0; vdx < Position.Num(); vdx++)
						{
							const Chaos::FVec3f& Pos = Evolution->Particles().GetX(vdx + Range.Start);
							FVector PosD = UEVertd(Pos);
							Position[vdx] = UEVertf(FinalXf.TransformPosition(PosD));
						}
					}
					else
					{
						for (int32 vdx = 0; vdx < Position.Num(); vdx++)
						{
							Position[vdx] = UEVertf(UEVertd(Evolution->Particles().GetX(vdx + Range.Start)));
						}
					}
				}
			}
		}
	}


	void FDeformableSolver::DebugDrawSimulationData()
	{
#if WITH_EDITOR
		auto ToFVec3 = [](FVector3d V) { return Chaos::FVec3(V.X, V.Y, V.Z); };
		auto ToFVector = [](Chaos::FVec3 V) { return FVector(V.X, V.Y, V.Z); };
		auto ToFQuat = [](const TRotation<FSolverReal, 3>& R) { return FQuat(R.X, R.Y, R.Z, R.W); };

		//debug draw
		//p.Chaos.DebugDraw.Enabled 1
		//p.Chaos.DebugDraw.Deformable.RigidCollisionGeometry 1
		if (Evolution && GDeformableDebugParams.bDoDrawRigidCollisionGeometry)
		{
			Evolution->CollisionParticlesActiveView().RangeFor(
				[this, ToFVec3, ToFVector, ToFQuat](FSolverCollisionParticles& CollisionParticles, int32 CollisionOffset, int32 CollisionRange)
				{
					for (int32 Index = CollisionOffset; Index < CollisionRange; Index++)
					{
						if (Evolution->CollisionParticleGroupIds()[Index] != Index)
						{
							if (const Chaos::FImplicitObjectPtr& Geometry = CollisionParticles.GetGeometry(Index))
							{
								EImplicitObjectType GeomType = Geometry->GetCollisionType();
								if (GeomType == ImplicitObjectType::Sphere)
								{
									const FSphere& SphereGeometry = Geometry->GetObjectChecked<FSphere>();
									FVector Center = ToFVector(CollisionParticles.GetX(Index)) + FVector(SphereGeometry.GetCenterf());

									FReal Radius = SphereGeometry.GetRadiusf();
									Chaos::FDebugDrawQueue::GetInstance().DrawDebugSphere(Center, Radius, 12, FColor::Red, false, -1.0f, 0, 1.f);
								}
								else if (GeomType == ImplicitObjectType::Box)
								{
									const TBox<FReal, 3>& BoxGeometry = Geometry->GetObjectChecked<TBox<FReal, 3>>();
									FVector Extent = 0.5 * (BoxGeometry.Max() - BoxGeometry.Min());
									FVector Center = ToFVector(CollisionParticles.GetX(Index)) + BoxGeometry.GetCenter();
									const FQuat& Rotation = ToFQuat(CollisionParticles.GetR(Index));
									Chaos::FDebugDrawQueue::GetInstance().DrawDebugBox(Center, Extent, Rotation, FColor::Red, false, -1.0f, 0, 1.f);
								}
								else if (GeomType == ImplicitObjectType::Convex)
								{
									const FConvex& ConvexGeometry = Geometry->GetObjectChecked<FConvex>();
									FTransform M = FTransform(ToFQuat(CollisionParticles.GetR(Index)), ToFVector(CollisionParticles.GetX(Index)));
									for (int32 EdgeIndex = 0; EdgeIndex < ConvexGeometry.NumEdges(); ++EdgeIndex)
									{
										int32 Index0 = ConvexGeometry.GetEdgeVertex(EdgeIndex, 0);
										int32 Index1 = ConvexGeometry.GetEdgeVertex(EdgeIndex, 1);
										const  TArray<FConvex::FVec3Type>& Verts = ConvexGeometry.GetVertices();
										Chaos::FDebugDrawQueue::GetInstance().DrawDebugLine(
											M.TransformPosition(ToFVector(Verts[Index0])), M.TransformPosition(ToFVector(Verts[Index1])), FColor::Red, false, -1.0f, 0, 1.f);
									}
								}
							}
						}
					}
				});
		}

		// p.Chaos.Deformable.NumLogExtremeParticle 
		if (Property.bUseGaussSeidelConstraints && GDeformableXPBDCorotatedParams.NumLogExtremeParticle > 0)
		{
			auto RestPosition = [this](int32 VertIdx, FVector3f& OutPosition)
				{
					if (const UObject* Owner = this->MObjects[VertIdx])
					{
						if (const FFleshThreadingProxy* Proxy = Proxies[Owner]->As<FFleshThreadingProxy>())
						{
							const FManagedArrayCollection& Rest = Proxy->GetRestCollection();
							const Chaos::FRange& Range = Proxy->GetSolverParticleRange();
							if (const TManagedArray<FVector3f>* Vertex = Rest.FindAttributeTyped<FVector3f>("Vertex", FGeometryCollection::VerticesGroup))
							{
								OutPosition = (*Vertex)[VertIdx - Range.Start];
								return true;
							}
						}
					}
					return false;
				};
			TArray<float> DistRatio;
			DistRatio.Init(0, AllIncidentElements->Num());
			TArray<int32> DistIndices;
			DistIndices.SetNum(AllIncidentElements->Num());
			for (int32 VertIdx = 0; VertIdx < AllIncidentElements->Num(); ++VertIdx)
			{
				DistIndices[VertIdx] = VertIdx;
				FVector3f VertPosition;
				if (!RestPosition(VertIdx, VertPosition))
				{
					continue;
				}

				TSet<int32> Neighbors;
				for (int32 IncidentIdx = 0; IncidentIdx < (*AllIncidentElements)[VertIdx].Num(); ++IncidentIdx)
				{
					for (int32 LocalTetIdx = 0; LocalTetIdx < 4; ++LocalTetIdx)
					{
						int32 Neighbor = (*AllElements)[(*AllIncidentElements)[VertIdx][IncidentIdx]][LocalTetIdx];
						if (Neighbor != VertIdx)
						{
							Neighbors.Add(Neighbor);
						}
					}
				}

				float TotalDist = 0;
				float TotalRest = 0;
				for (int32 NeighborIdx : Neighbors)
				{
					FVector3f NeighborPosition;
					if (!RestPosition(NeighborIdx, NeighborPosition))
					{
						continue;
					}
					//Particles().X is already updated after AdvanceDt
					float Dist = (Evolution->Particles().GetX(VertIdx) - Evolution->Particles().GetX(NeighborIdx)).Size();
					TotalDist += Dist;
					float Rest = (VertPosition - NeighborPosition).Size();
					TotalRest += Rest;
				}
				if (TotalRest > UE_SMALL_NUMBER)
				{
					DistRatio[VertIdx] = TotalDist / TotalRest;
				}
			}
			DistIndices.Sort([&DistRatio](int32 A, int32 B)
				{
					return DistRatio[A] > DistRatio[B]; // Descending order
				});
			for (int32 i = 0; i < FMath::Min(GDeformableXPBDCorotatedParams.NumLogExtremeParticle, DistIndices.Num()); ++i)
			{
				UE_LOGF(LogChaosDeformableSolver, Warning, "Particle index %d has average deformation ratio %f", DistIndices[i], DistRatio[DistIndices[i]]);
			}
		}
#endif
	}


	void FDeformableSolver::WriteFrame(FThreadingProxy& InProxy, const FSolverReal DeltaTime)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ChaosDeformableSolver_WriteFrame);
		if (FFleshThreadingProxy* Proxy = InProxy.As<FFleshThreadingProxy>())
		{
			const FManagedArrayCollection& Rest = Proxy->GetRestCollection();
			if (SurfaceElements)
			{
				WriteTrisGEO(Evolution->Particles(), *SurfaceElements);
				FString file = FPaths::ProjectDir();
				file.Append(TEXT("/DebugOutput/DtLog.txt"));
				if (Frame == 0)
				{
					FFileHelper::SaveStringToFile(FString("DeltaTime\r\n"), *file);
				}
				FFileHelper::SaveStringToFile((FString::SanitizeFloat(DeltaTime) + FString("\r\n")), *file, FFileHelper::EEncodingOptions::AutoDetect, &IFileManager::Get(), EFileWrite::FILEWRITE_Append);
			}
		}
	}

	void FDeformableSolver::WriteTrisGEO(const FSolverParticles& Particles, const TArray<TVec3<int32>>& Mesh)
	{
		FString file = FPaths::ProjectDir();
		file.Append(TEXT("/DebugOutput/sim_frame_"));
		file.Append(FString::FromInt(Frame));
		file.Append(TEXT(".geo"));

		int32 Np = Particles.Size();
		int32 NPrims = Mesh.Num();

		// We will use this FileManager to deal with the file.
		IPlatformFile& FileManager = FPlatformFileManager::Get().GetPlatformFile();
		FFileHelper::SaveStringToFile(FString("PGEOMETRY V5\r\n"), *file);
		FString HeaderInfo = FString("NPoints ") + FString::FromInt(Np) + FString(" NPrims ") + FString::FromInt(NPrims) + FString("\r\n");
		FString MoreHeader = FString("NPointGroups 0 NPrimGroups 0\r\nNPointAttrib 0 NVertexAttrib 0 NPrimAttrib 0 NAttrib 0\r\n");

		FFileHelper::SaveStringToFile(HeaderInfo, *file, FFileHelper::EEncodingOptions::AutoDetect, &IFileManager::Get(), EFileWrite::FILEWRITE_Append);
		FFileHelper::SaveStringToFile(MoreHeader, *file, FFileHelper::EEncodingOptions::AutoDetect, &IFileManager::Get(), EFileWrite::FILEWRITE_Append);

		for (int32 i = 0; i < Np; i++) {

			FString ParticleInfo = FString::SanitizeFloat(Particles.GetX(i)[0]) + FString(" ") + FString::SanitizeFloat(Particles.GetX(i)[1]) + FString(" ") + FString::SanitizeFloat(Particles.GetX(i)[2]) + FString(" ") + FString::FromInt(1) + FString("\r\n");
			FFileHelper::SaveStringToFile(ParticleInfo, *file, FFileHelper::EEncodingOptions::AutoDetect, &IFileManager::Get(), EFileWrite::FILEWRITE_Append);

		}

		for (int32 i = 0; i < Mesh.Num(); i++) {
			//outstream << "Poly 3 < ";
			FString ElementToWrite = FString("Poly 3 < ") + FString::FromInt(Mesh[i][0]) + FString(" ") + FString::FromInt(Mesh[i][1]) + FString(" ") + FString::FromInt(Mesh[i][2]) + FString("\r\n");
			FFileHelper::SaveStringToFile(ElementToWrite, *file, FFileHelper::EEncodingOptions::AutoDetect, &IFileManager::Get(), EFileWrite::FILEWRITE_Append);
		}

		FFileHelper::SaveStringToFile(FString("beginExtra\n"), *file, FFileHelper::EEncodingOptions::AutoDetect, &IFileManager::Get(), EFileWrite::FILEWRITE_Append);
		FFileHelper::SaveStringToFile(FString("endExtra\n"), *file, FFileHelper::EEncodingOptions::AutoDetect, &IFileManager::Get(), EFileWrite::FILEWRITE_Append);

	}


}; // Namespace Chaos::Softs
