// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/Deformable/ChaosDeformableSolver.h"
#include "Chaos/Deformable/ChaosDeformableSolverTypes.h"
#include "Chaos/Deformable/ChaosDeformableSolverProxy.h"

#include "Chaos/TriangleMesh.h"
#include "Chaos/PBDAltitudeSpringConstraints.h"
#include "Chaos/PBDBendingConstraints.h"
#include "Chaos/PBDSpringConstraints.h"
#include "Chaos/PBDVolumeConstraint.h"
#include "Chaos/PBDSpringConstraints.h"
#include "Chaos/PBDTriangleMeshCollisions.h"
#include "Chaos/PBDCollisionSpringConstraints.h"
#include "Chaos/PBDParticles.h"
#include "Chaos/PBDSoftsSolverParticles.h"
#include "Chaos/PBDTetConstraints.h"
#include "Chaos/PerParticleGravity.h"
#include "Chaos/XPBDCorotatedConstraints.h"
#include "Chaos/XPBDVolumeConstraints.h"
#include "Chaos/XPBDCorotatedFiberConstraints.h"
#include "Chaos/Plane.h"
#include "Chaos/Utilities.h"
#include "Chaos/PBDEvolution.h"
#include "Containers/StringConv.h"
#include "CoreMinimal.h"
#include "GeometryCollection/Facades/CollectionKinematicBindingFacade.h"
#include "Misc/FileHelper.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/IConsoleManager.h"
#include "Misc/Paths.h"


namespace Chaos::Softs
{
	void FDeformableSolver::FPhysicsThreadAccess::Reset(const FDeformableSolverProperties& InProps)
	{
		if(Solver) 
			Solver->Reset(InProps);
	}

	void FDeformableSolver::FPhysicsThreadAccess::LoadRestartData()
	{
		if (Solver)
		{
			Solver->LoadRestartData();
		}
	}

	void FDeformableSolver::FPhysicsThreadAccess::AdvanceDt(FSolverReal DeltaTime)
	{
		if (Solver)
			Solver->AdvanceDt(DeltaTime);
	}

	void FDeformableSolver::FPhysicsThreadAccess::Simulate(FSolverReal DeltaTime)
	{
		if(Solver) 
			Solver->Simulate(DeltaTime);
	}

	void FDeformableSolver::FPhysicsThreadAccess::InitializeSimulationObjects()
	{
		if(Solver) 
			Solver->InitializeSimulationObjects();
	}

	void FDeformableSolver::FPhysicsThreadAccess::InitializeSimulationObject(FThreadingProxy& InProxy)
	{
		if(Solver) 
			Solver->InitializeSimulationObject(InProxy);
	}

	void FDeformableSolver::FPhysicsThreadAccess::InitializeKinematicConstraint()
	{
		if(Solver) Solver->InitializeKinematicConstraint();
	}

	void FDeformableSolver::FPhysicsThreadAccess::InitializeSelfCollisionVariables()
	{
		if(Solver) 
			Solver->InitializeSelfCollisionVariables();
	}

	bool FDeformableSolver::FGameThreadAccess::HasObject(UObject* InObject) const
	{
		if (Solver)
			return Solver->HasObject(InObject);
		return false;
	}

	void FDeformableSolver::FGameThreadAccess::PushInputPackage(int32 InFrame, FDeformableDataMap&& InPackage)
	{
		if(Solver) 
			Solver->PushInputPackage(InFrame, MoveTemp(InPackage));
	}

	void FDeformableSolver::FGameThreadAccess::PushRestartPackage(int32 InFrame, FDeformableDataMap&& InPackage)
	{
		if (Solver)
		{
			Solver->PushRestartPackage(InFrame, MoveTemp(InPackage));
		}	
	}

	TUniquePtr<FDeformablePackage> FDeformableSolver::FPhysicsThreadAccess::PullInputPackage()
	{
		if (Solver)
			return Solver->PullInputPackage();
		return TUniquePtr<FDeformablePackage>(nullptr);
	}

	void FDeformableSolver::FPhysicsThreadAccess::UpdateProxyInputPackages()
	{
		if (Solver)
			Solver->UpdateProxyInputPackages();
	}

	void FDeformableSolver::FPhysicsThreadAccess::RemoveSimulationObjects()
	{
		if(Solver) 
			Solver->RemoveSimulationObjects();
	}

	void FDeformableSolver::FPhysicsThreadAccess::Update(FSolverReal DeltaTime)
	{
		if(Solver)
			Solver->Update(DeltaTime);
	}

	void FDeformableSolver::FPhysicsThreadAccess::PushOutputPackage(int32 InFrame, FDeformableDataMap&& InPackage)
	{
		if(Solver)
			Solver->PushOutputPackage(InFrame, MoveTemp(InPackage));
	}

	TUniquePtr<FDeformablePackage>  FDeformableSolver::FGameThreadAccess::PullOutputPackage()
	{
		if (Solver)
			return  Solver->PullOutputPackage();
		return TUniquePtr<FDeformablePackage>(nullptr);
	}

	void  FDeformableSolver::FGameThreadAccess::AddProxy(FThreadingProxy* InObject)
	{
		if (Solver)
			Solver->AddProxy(InObject);
	}

	void  FDeformableSolver::FGameThreadAccess::RemoveProxy(FThreadingProxy* InObject)
	{
		if(Solver) 
			Solver->RemoveProxy(InObject);
	}

	void FDeformableSolver::FGameThreadAccess::SetEnableSolver(bool InbEnableSolver)
	{
		if (Solver)
			Solver->SetEnableSolver(InbEnableSolver);
	}

	bool FDeformableSolver::FGameThreadAccess::GetEnableSolver()
	{
		if (Solver)
			return Solver->GetEnableSolver();
		return false;
	}

	void  FDeformableSolver::FPhysicsThreadAccess::UpdateOutputState(FThreadingProxy& InProxy)
	{
		if(Solver) 
			Solver->UpdateOutputState(InProxy);
	}

	TArray<float> FDeformableSolver::FPhysicsThreadAccess::GetParticleMuscleActivation()
	{
		TArray<float> ParticleMuscleActivations;
		ParticleMuscleActivations.Init(-1.f, Solver->Evolution->Particles().Size());
		if (Solver->MuscleActivationConstraints)
		{
			for (int32 MuscleIndex = 0; MuscleIndex < Solver->MuscleActivationConstraints->NumMuscles(); ++MuscleIndex)
			{
				const int32 VertexOffset = Solver->MuscleActivationConstraints->GetMuscleVertexOffset(MuscleIndex);
				const int32 VertexCount = Solver->MuscleActivationConstraints->GetMuscleVertexCount(MuscleIndex);
				for (int32 VertexIdx = VertexOffset; VertexIdx < VertexOffset + VertexCount; ++VertexIdx)
				{
					ParticleMuscleActivations[VertexIdx] = Solver->MuscleActivationConstraints->GetMuscleActivation(MuscleIndex);
				}
			}
		}
		return ParticleMuscleActivations;
	}
}; // Namespace Chaos::Softs