// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/Deformable/GaussSeidelCorotatedCodimensionalConstraints.h"

namespace Chaos::Softs
{

FGaussSeidelCorotatedTriangleConstraints::FGaussSeidelCorotatedTriangleConstraints(
	const FSolverParticlesRange& InParticles,
	const TArray<TVector<int32, 3>>& InMesh,
	const bool bRecordMetricIn,
	const FSolverReal& EMesh,
	const FSolverReal& NuMesh)
	: bRecordMetric(bRecordMetricIn), MeshConstraints(InMesh)
{
	Measure.Init((FSolverReal)0., MeshConstraints.Num());
	Lambda = EMesh * NuMesh / (((FSolverReal)1. + NuMesh) * ((FSolverReal)1. - (FSolverReal)2. * NuMesh));
	Mu = EMesh / ((FSolverReal)2. * ((FSolverReal)1. + NuMesh));

	MuElementArray.Init(Mu, MeshConstraints.Num());
	LambdaElementArray.Init(Lambda, MeshConstraints.Num());

	InitializeCodimensionData(InParticles);
}

FGaussSeidelCorotatedTriangleConstraints::FGaussSeidelCorotatedTriangleConstraints(
	const FSolverParticlesRange& InParticles,
	const TArray<TVector<int32, 3>>& InMesh,
	const TArray<FSolverReal>& EMeshArray,
	const FSolverReal& NuMesh)
	: MeshConstraints(InMesh)
{
	ensureMsgf(EMeshArray.Num() == InMesh.Num(), TEXT("Input Young Modulus Array Size is wrong"));
	Measure.Init((FSolverReal)0., MeshConstraints.Num());

	InitializeCodimensionData(InParticles);
	LambdaElementArray.Init((FSolverReal)0., MeshConstraints.Num());
	MuElementArray.Init((FSolverReal)0., MeshConstraints.Num());

	for (int32 ElemIdx = 0; ElemIdx < InMesh.Num(); ElemIdx++)
	{
		LambdaElementArray[ElemIdx] = EMeshArray[ElemIdx] * NuMesh / (((FSolverReal)1. + NuMesh) * ((FSolverReal)1. - (FSolverReal)2. * NuMesh));
		MuElementArray[ElemIdx] = EMeshArray[ElemIdx] / ((FSolverReal)2. * ((FSolverReal)1. + NuMesh));
	}
}

PMatrix<FSolverReal, 3, 2> FGaussSeidelCorotatedTriangleConstraints::Ds(const int32 ElemIdx, const FSolverParticlesRange& InParticles) const
{
	if (INDEX_NONE < ElemIdx && ElemIdx < MeshConstraints.Num()
		&& INDEX_NONE < MeshConstraints[ElemIdx][0] && MeshConstraints[ElemIdx][0] < InParticles.Size()
		&& INDEX_NONE < MeshConstraints[ElemIdx][1] && MeshConstraints[ElemIdx][1] < InParticles.Size()
		&& INDEX_NONE < MeshConstraints[ElemIdx][2] && MeshConstraints[ElemIdx][2] < InParticles.Size())
	{
		const TVec3<FSolverReal> P1P0 = InParticles.P(MeshConstraints[ElemIdx][1]) - InParticles.P(MeshConstraints[ElemIdx][0]);
		const TVec3<FSolverReal> P2P0 = InParticles.P(MeshConstraints[ElemIdx][2]) - InParticles.P(MeshConstraints[ElemIdx][0]);

		return PMatrix<FSolverReal, 3, 2>(
			P1P0[0], P1P0[1], P1P0[2],
			P2P0[0], P2P0[1], P2P0[2]);
	}
	else
	{
		return PMatrix<FSolverReal, 3, 2>(
			(FSolverReal)0., (FSolverReal)0, (FSolverReal)0,
			(FSolverReal)0, (FSolverReal)0, (FSolverReal)0);
	}
}

PMatrix<FSolverReal, 3, 2> FGaussSeidelCorotatedTriangleConstraints::F(const int32 ElemIdx, const FSolverParticlesRange& InParticles) const
{
	if (INDEX_NONE < ElemIdx && ElemIdx < DmInverse.Num())
	{
		return Ds(ElemIdx, InParticles) * DmInverse[ElemIdx];
	}
	else
	{
		return PMatrix<FSolverReal, 3, 2>(
			(FSolverReal)0., (FSolverReal)0, (FSolverReal)0,
			(FSolverReal)0, (FSolverReal)0, (FSolverReal)0);
	}
}

TArray<TArray<int32>> FGaussSeidelCorotatedTriangleConstraints::GetConstraintsArray() const
{
	TArray<TArray<int32>> Constraints;
	Constraints.SetNum(MeshConstraints.Num());
	for (int32 ElemIdx = 0; ElemIdx < MeshConstraints.Num(); ElemIdx++)
	{
		Constraints[ElemIdx].SetNum(3);
		for (int32 TriLocalIdx = 0; TriLocalIdx < 3; TriLocalIdx++)
		{
			Constraints[ElemIdx][TriLocalIdx] = MeshConstraints[ElemIdx][TriLocalIdx];
		}
	}
	return Constraints;
}

void FGaussSeidelCorotatedTriangleConstraints::ComputeNodalMass(const FSolverReal InDensity, const int32 NumParticles, TArray<FSolverReal>& NodalMass) const
{
	NodalMass.Init((FSolverReal)0., NumParticles);
	for (int32 ElemIdx = 0; ElemIdx < Measure.Num(); ElemIdx++)
	{
		const FSolverReal TotalMass = InDensity * Measure[ElemIdx];
		for (int32 TriLocalIdx = 0; TriLocalIdx < 3; TriLocalIdx++)
		{
			NodalMass[MeshConstraints[ElemIdx][TriLocalIdx]] += TotalMass / (FSolverReal)3.;
		}
	}
}

void FGaussSeidelCorotatedTriangleConstraints::InitializeCodimensionData(const FSolverParticlesRange& Particles)
{
	Measure.Init((FSolverReal)0., MeshConstraints.Num());
	DmInverse.Init(PMatrix<FSolverReal, 2, 2>(0.f, 0.f, 0.f), MeshConstraints.Num());
	for (int32 ElemIdx = 0; ElemIdx < MeshConstraints.Num(); ElemIdx++)
	{
		if (!ensure(MeshConstraints[ElemIdx][0] > INDEX_NONE && MeshConstraints[ElemIdx][0] < Particles.Size()
			&& MeshConstraints[ElemIdx][1] > INDEX_NONE && MeshConstraints[ElemIdx][1] < Particles.Size()
			&& MeshConstraints[ElemIdx][2] > INDEX_NONE && MeshConstraints[ElemIdx][2] < Particles.Size()))
		{
			continue;
		}

		const TVec3<FSolverReal> X1X0 = Particles.GetX(MeshConstraints[ElemIdx][1]) - Particles.GetX(MeshConstraints[ElemIdx][0]);
		const TVec3<FSolverReal> X2X0 = Particles.GetX(MeshConstraints[ElemIdx][2]) - Particles.GetX(MeshConstraints[ElemIdx][0]);
		PMatrix<FSolverReal, 2, 2> Dm((FSolverReal)0., (FSolverReal)0., (FSolverReal)0.);
		Dm.M[0] = X1X0.Size();
		Dm.M[2] = X1X0.Dot(X2X0) / Dm.M[0];
		Dm.M[3] = Chaos::TVector<FSolverReal, 3>::CrossProduct(X1X0, X2X0).Size() / Dm.M[0];
		Measure[ElemIdx] = Chaos::TVector<FSolverReal, 3>::CrossProduct(X1X0, X2X0).Size() / (FSolverReal)2.;
		ensureMsgf(Measure[ElemIdx] > (FSolverReal)0., TEXT("Degenerate triangle detected"));

		const PMatrix<FSolverReal, 2, 2> DmInv = Dm.Inverse();
		DmInverse[ElemIdx] = DmInv;
	}
}

PMatrix<FSolverReal, 3, 2> FGaussSeidelCorotatedTriangleConstraints::ComputeR(const PMatrix<FSolverReal, 3, 2>& Fe)
{
	PMatrix<FSolverReal, 2, 2> FTF = ComputeFTF(Fe), D((FSolverReal)0., (FSolverReal)0., (FSolverReal)0.), U((FSolverReal)0., (FSolverReal)0., (FSolverReal)0.);
	Jacobi(FTF, D, U);
	const PMatrix<FSolverReal, 2, 2> SDiag(FMath::Sqrt(D.M[0]), (FSolverReal)0., (FSolverReal)0., FMath::Sqrt(D.M[3]));
	const PMatrix<FSolverReal, 2, 2> S = U * SDiag * U.GetTransposed();
	return Fe * S.Inverse();
}

PMatrix<FSolverReal, 3, 2> FGaussSeidelCorotatedTriangleConstraints::ComputeRSimple(const PMatrix<FSolverReal, 3, 2>& InputMatrix)
{
	TVec3<FSolverReal> Col1(InputMatrix.M[0], InputMatrix.M[1], InputMatrix.M[2]), Col2(InputMatrix.M[3], InputMatrix.M[4], InputMatrix.M[5]), A(FSolverReal(0)), B(FSolverReal(0));
	if (FMath::Abs(Col1[0]) < (FSolverReal)UE_SMALL_NUMBER && FMath::Abs(Col1[1]) < (FSolverReal)UE_SMALL_NUMBER)
	{
		A[0] = FSolverReal(1);
	}
	else
	{
		A[0] = -Col1[1];
		A[1] = Col1[0];
		const FSolverReal OneOverANorm = SafeRecip(A.Length(), 0.f);
		A *= OneOverANorm;
	}
	B = TVec3<FSolverReal>::CrossProduct(A, Col2);
	const FSolverReal OneOverBNorm = SafeRecip(B.Length(), 0.f);
	B *= OneOverBNorm;
	return Chaos::PMatrix<FSolverReal, 3, 2>(B[0], B[1], B[2], A[0], A[1], A[2]);
}

void FGaussSeidelCorotatedTriangleConstraints::AddHyperelasticResidualAndHessian(const FSolverParticlesRange& Particles, const int32 ElementIndex, const int32 ElementIndexLocal, const FSolverReal Dt, TVec3<FSolverReal>& ParticleResidual, Chaos::PMatrix<FSolverReal, 3, 3>& ParticleHessian)
{
	if (!ensure(ElementIndex > INDEX_NONE
		&& ElementIndex < DmInverse.Num()
		&& ElementIndex < MuElementArray.Num()
		&& ElementIndex < Measure.Num()
		&& ElementIndex < LambdaElementArray.Num()
		&& ElementIndexLocal > INDEX_NONE
		&& ElementIndexLocal < 3))
	{
		return;
	}

	const Chaos::PMatrix<FSolverReal, 2, 2> DmInvT = DmInverse[ElementIndex].GetTransposed(), DmInv = DmInverse[ElementIndex];
	const Chaos::PMatrix<FSolverReal, 3, 2> Fe = F(ElementIndex, Particles);

	const PMatrix<FSolverReal, 3, 2> Re = ComputeR(Fe);
	const PMatrix<FSolverReal, 2, 2> FTF = ComputeFTF(Fe);
	const FSolverReal J = FMath::Sqrt(FTF.Determinant());

	PMatrix<FSolverReal, 3, 2> Pe(TVec3<FSolverReal>((FSolverReal)0.), TVec3<FSolverReal>((FSolverReal)0.)), ForceTerm(TVec3<FSolverReal>((FSolverReal)0.), TVec3<FSolverReal>((FSolverReal)0.));
	Pe = FSolverReal(2) * MuElementArray[ElementIndex] * (Fe - Re) + LambdaElementArray[ElementIndex] * (J - FSolverReal(1)) * J * Fe * ComputeFTF(Fe).Inverse();

	ForceTerm = -Measure[ElementIndex] * Pe * DmInverse[ElementIndex].GetTransposed();

	Chaos::TVector<FSolverReal, 3> Dx((FSolverReal)0.);
	if (ElementIndexLocal > 0)
	{
		for (int32 Alpha = 0; Alpha < 3; Alpha++)
		{
			Dx[Alpha] += ForceTerm.M[ElementIndexLocal * 3 - 3 + Alpha];
		}
	}
	else
	{
		for (int32 Alpha = 0; Alpha < 3; Alpha++)
		{
			for (int32 Col = 0; Col < 2; Col++)
			{
				Dx[Alpha] -= ForceTerm.M[Col * 3 + Alpha];
			}
		}
	}

	Dx *= Dt * Dt;

	ParticleResidual -= Dx;

	PMatrix<FSolverReal, 3, 2> DeltaJ(TVec3<FSolverReal>((FSolverReal)0.), TVec3<FSolverReal>((FSolverReal)0.));

	dJdF32(Fe, DeltaJ);

	const FSolverReal Coeff = Dt * Dt * Measure[ElementIndex];

	if (ElementIndexLocal == 0)
	{
		FSolverReal DmInvSum = FSolverReal(0);
		for (int32 Row = 0; Row < 2; Row++)
		{
			FSolverReal LocalDmSum = FSolverReal(0);
			for (int32 Col = 0; Col < 2; Col++)
			{
				LocalDmSum += DmInv.M[Row * 2 + Col];
			}
			DmInvSum += LocalDmSum * LocalDmSum;
		}
		for (int32 Alpha = 0; Alpha < 3; Alpha++)
		{
			ParticleHessian.SetAt(Alpha, Alpha, ParticleHessian.GetAt(Alpha, Alpha) + Coeff * FSolverReal(2) * MuElementArray[ElementIndex] * DmInvSum);
		}

		const PMatrix<FSolverReal, 3, 2> DeltaJDmInvT = DeltaJ * DmInvT;
		Chaos::TVector<FSolverReal, 3> L((FSolverReal)0.);
		for (int32 Alpha = 0; Alpha < 3; Alpha++)
		{
			for (int32 Col = 0; Col < 2; Col++)
			{
				L[Alpha] += DeltaJDmInvT.M[3 * Col + Alpha];
			}
		}
		for (int32 Alpha = 0; Alpha < 3; Alpha++)
		{
			ParticleHessian.SetRow(Alpha, ParticleHessian.GetRow(Alpha) + Coeff * LambdaElementArray[ElementIndex] * L[Alpha] * L);
		}
	}
	else
	{
		FSolverReal DmInvSum = FSolverReal(0);
		for (int32 Col = 0; Col < 2; Col++)
		{
			DmInvSum += DmInv.GetAt(ElementIndexLocal - 1, Col) * DmInv.GetAt(ElementIndexLocal - 1, Col);
		}
		for (int32 Alpha = 0; Alpha < 3; Alpha++)
		{
			ParticleHessian.SetAt(Alpha, Alpha, ParticleHessian.GetAt(Alpha, Alpha) + Dt * Dt * Measure[ElementIndex] * FSolverReal(2) * MuElementArray[ElementIndex] * DmInvSum);
		}

		const PMatrix<FSolverReal, 3, 2> DeltaJDmInvT = DeltaJ * DmInvT;

		Chaos::TVector<FSolverReal, 3> L((FSolverReal)0.);
		for (int32 Alpha = 0; Alpha < 3; Alpha++)
		{
			const int32 IndexVisited = ElementIndexLocal * 3 - 3 + Alpha;
			if (IndexVisited < 6 && IndexVisited > INDEX_NONE)
			{
				L[Alpha] = DeltaJDmInvT.M[IndexVisited];
			}
		}
		for (int32 Alpha = 0; Alpha < 3; Alpha++)
		{
			ParticleHessian.SetRow(Alpha, ParticleHessian.GetRow(Alpha) + Dt * Dt * Measure[ElementIndex] * LambdaElementArray[ElementIndex] * L[Alpha] * L);
		}
	}
}

}  // End namespace Chaos::Softs
