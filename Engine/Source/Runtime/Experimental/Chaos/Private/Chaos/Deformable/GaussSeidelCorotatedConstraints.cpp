// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/Deformable/GaussSeidelCorotatedConstraints.h"

namespace Chaos::Softs
{

FGaussSeidelCorotatedTetrahedralConstraints::FGaussSeidelCorotatedTetrahedralConstraints(
	const FSolverParticles& InParticles,
	const TArray<TVector<int32, 4>>& InMesh,
	const TArray<FSolverReal>& EMeshArray,
	const TArray<FSolverReal>& NuMeshArray,
	TArray<FSolverReal>&& AlphaJMeshArray,
	TArray<TArray<int32>>&& IncidentElementsIn,
	TArray<TArray<int32>>&& IncidentElementsLocalIn,
	const int32 ParticleStartIndexIn,
	const int32 ParticleEndIndexIn,
	const bool bDoQuasistaticsIn,
	const bool bDoSORIn,
	const FSolverReal InOmegaSOR,
	const FDeformableXPBDCorotatedParams& InParams,
	const FSolverReal& NuMesh,
	const bool bRecordMetricIn)
	: FGaussSeidelCorotatedTetrahedralConstraints(
		FSolverParticlesRange(InParticles),
		InMesh, EMeshArray, NuMeshArray, MoveTemp(AlphaJMeshArray),
		MoveTemp(IncidentElementsIn), MoveTemp(IncidentElementsLocalIn),
		ParticleStartIndexIn, ParticleEndIndexIn,
		bDoQuasistaticsIn, bDoSORIn, InOmegaSOR, InParams, NuMesh, bRecordMetricIn)
{}

FGaussSeidelCorotatedTetrahedralConstraints::FGaussSeidelCorotatedTetrahedralConstraints(
	const FSolverParticlesRange& InParticles,
	const TArray<TVector<int32, 4>>& InMesh,
	const TArray<FSolverReal>& EMeshArray,
	const TArray<FSolverReal>& NuMeshArray,
	TArray<FSolverReal>&& AlphaJMeshArray,
	TArray<TArray<int32>>&& IncidentElementsIn,
	TArray<TArray<int32>>&& IncidentElementsLocalIn,
	const int32 ParticleStartIndexIn,
	const int32 ParticleEndIndexIn,
	const bool bDoQuasistaticsIn,
	const bool bDoSORIn,
	const FSolverReal InOmegaSOR,
	const FDeformableXPBDCorotatedParams& InParams,
	const FSolverReal& NuMesh,
	const bool bRecordMetricIn)
	: Base(InParticles, InMesh, EMeshArray, NuMeshArray, MoveTemp(AlphaJMeshArray), InParams, NuMesh, bRecordMetricIn, false), IncidentElements(IncidentElementsIn), IncidentElementsLocal(IncidentElementsLocalIn),
	ParticleStartIndex(ParticleStartIndexIn), ParticleEndIndex(ParticleEndIndexIn), bDoQuasistatics(bDoQuasistaticsIn), bDoSOR(bDoSORIn), OmegaSOR(InOmegaSOR)
{
	Base::LambdaArray.SetNum(0);
	for (int32 ElemIdx = 0; ElemIdx < InMesh.Num(); ElemIdx++)
	{
		for (int32 Row = 0; Row < 3; Row++)
		{
			for (int32 Col = 0; Col < 3; Col++)
			{
				DmInverse[(3 * 3) * ElemIdx + 3 * Row + Col] /= Base::AlphaJArray[ElemIdx];
			}
		}
	}
	if (!bDoQuasistatics)
	{
		xtilde.Init(TVector<FSolverReal, 3>(FSolverReal(0.)), ParticleEndIndex - ParticleStartIndex);
	}
	if (bDoSOR)
	{
		X_k_1.Init(TVector<FSolverReal, 3>(FSolverReal(0.)), ParticleEndIndex - ParticleStartIndex);
		X_k.Init(TVector<FSolverReal, 3>(FSolverReal(0.)), ParticleEndIndex - ParticleStartIndex);
	}
	InitColor(InParticles);
	InitializeCorotatedLambdas();
}

void FGaussSeidelCorotatedTetrahedralConstraints::Apply(FSolverParticlesRange& Particles, const FSolverReal Dt) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(STAT_ChaosGaussSeidelApply);
	for (int32 ColorIdx = 0; ColorIdx < ParticlesPerColor.Num(); ColorIdx++)
	{
		int32 NumBatch = ParticlesPerColor[ColorIdx].Num() / CorotatedParams.XPBDCorotatedBatchSize;
		if (ParticlesPerColor[ColorIdx].Num() % CorotatedParams.XPBDCorotatedBatchSize != 0)
		{
			NumBatch += 1;
		}

		PhysicsParallelFor(NumBatch, [&](const int32 BatchIndex)
			{
				for (int32 BatchSubIndex = 0; BatchSubIndex < CorotatedParams.XPBDCorotatedBatchSize; BatchSubIndex++)
				{
					const int32 TaskIndex = CorotatedParams.XPBDCorotatedBatchSize * BatchIndex + BatchSubIndex;
					if (TaskIndex < ParticlesPerColor[ColorIdx].Num())
					{
						const int32 ParticleIndex = ParticlesPerColor[ColorIdx][TaskIndex];
						const Chaos::TVector<FSolverReal, 3> Dx = ComputeDeltax(ParticleIndex, ParticleIndex - ParticleStartIndex, Particles,
							Dt);
						Particles.P(ParticleIndex) += Dx;
					}
				}
			}, NumBatch < CorotatedParams.XPBDCorotatedBatchThreshold);
	}
	ApplySOR(Particles, Dt);
	CurrentIt += 1;
}

void FGaussSeidelCorotatedTetrahedralConstraints::Apply(FSolverParticles& Particles, const FSolverReal Dt) const
{
	FSolverParticlesRange Range(Particles);
	Apply(Range, Dt);
}

void FGaussSeidelCorotatedTetrahedralConstraints::Init(const FSolverReal Dt, const FSolverParticlesRange& Particles) const
{
	if (!bDoQuasistatics)
	{
		for (int32 LocalParticleIdx = 0; LocalParticleIdx < ParticleEndIndex - ParticleStartIndex; LocalParticleIdx++)
		{
			xtilde[LocalParticleIdx] = Particles.X(ParticleStartIndex + LocalParticleIdx) + Dt * Particles.V(ParticleStartIndex + LocalParticleIdx);
		}
	}
	CurrentIt = 0;
}

void FGaussSeidelCorotatedTetrahedralConstraints::Init(const FSolverReal Dt, const FSolverParticles& Particles) const
{
	Init(Dt, FSolverParticlesRange(Particles));
}

TArray<TArray<int32>> FGaussSeidelCorotatedTetrahedralConstraints::GetMeshArray() const
{
	TArray<TArray<int32>> MeshArray;
	MeshArray.SetNum(MeshConstraints.Num());
	for (int32 ElemIdx = 0; ElemIdx < MeshConstraints.Num(); ElemIdx++)
	{
		MeshArray[ElemIdx].SetNum(4);
		for (int32 TetLocalIdx = 0; TetLocalIdx < 4; TetLocalIdx++)
		{
			MeshArray[ElemIdx][TetLocalIdx] = MeshConstraints[ElemIdx][TetLocalIdx];
		}
	}
	return MeshArray;
}

void FGaussSeidelCorotatedTetrahedralConstraints::ApplySOR(FSolverParticlesRange& Particles, const FSolverReal Dt) const
{
	if (bDoSOR)
	{
		PhysicsParallelFor(ParticleEndIndex - ParticleStartIndex, [&](const int32 Index)
			{
				const int32 ParticleIndex = Index + ParticleStartIndex;
				if (Particles.InvM(ParticleIndex) != FSolverReal(0) && CurrentIt > 3)
				{
					Particles.P(ParticleIndex) = OmegaSOR * (Particles.P(ParticleIndex) - X_k_1[Index]) + X_k_1[Index];
				}
				X_k_1[Index] = X_k[Index];
				X_k[Index] = Particles.P(ParticleIndex);
			}, ParticleEndIndex - ParticleStartIndex < 1000);
	}
}

Chaos::TVector<FSolverReal, 3> FGaussSeidelCorotatedTetrahedralConstraints::ComputePerParticleResidual(const int32 ParticleIdx, const int32 IncidentIndex, const FSolverParticlesRange& Particles,
	const FSolverReal Dt, const bool AddMass) const
{
	Chaos::TVector<FSolverReal, 3> Residual(FSolverReal(0));
	if (AddMass)
	{
		for (int32 Alpha = 0; Alpha < 3; Alpha++)
		{
			Residual[Alpha] = Particles.M(ParticleIdx) * (Particles.P(ParticleIdx)[Alpha] - xtilde[ParticleIdx - ParticleStartIndex][Alpha]);
		}
	}
	for (int32 IncidentIdx = 0; IncidentIdx < IncidentElements[IncidentIndex].Num(); IncidentIdx++)
	{
		const int32 LocalIndex = IncidentElementsLocal[IncidentIndex][IncidentIdx];
		const int32 ElemIdx = IncidentElements[IncidentIndex][IncidentIdx];
		const Chaos::PMatrix<FSolverReal, 3, 3> DmInvT = Base::ElementDmInv(ElemIdx).GetTransposed();
		const Chaos::PMatrix<FSolverReal, 3, 3> Fe = Base::F(ElemIdx, Particles);
		Chaos::PMatrix<FSolverReal, 3, 3> P((FSolverReal)0.);

		ComputeStress(Fe, MuElementArray[ElemIdx], LambdaElementArray[ElemIdx], P);

		const Chaos::PMatrix<FSolverReal, 3, 3> ForceTerm = -Measure[ElemIdx] * DmInvT * P;
		Chaos::TVector<FSolverReal, 3> Dx((FSolverReal)0.);
		if (LocalIndex > 0)
		{
			for (int32 Alpha = 0; Alpha < 3; Alpha++)
			{
				Dx[Alpha] += ForceTerm.GetAt(Alpha, LocalIndex - 1);
			}
		}
		else
		{
			for (int32 Alpha = 0; Alpha < 3; Alpha++)
			{
				for (int32 Col = 0; Col < 3; Col++)
				{
					Dx[Alpha] -= ForceTerm.GetAt(Alpha, Col);
				}
			}
		}

		Dx *= Dt * Dt;

		for (int32 Alpha = 0; Alpha < 3; Alpha++)
		{
			Residual[Alpha] -= Dx[Alpha];
		}
	}
	return Residual;
}

Chaos::PMatrix<FSolverReal, 3, 3> FGaussSeidelCorotatedTetrahedralConstraints::ComputePerParticleCorotatedHessianSimple(const int32 ParticleIdx, const int32 IncidentIndex, const FSolverParticlesRange& Particles,
	const FSolverReal Dt, const bool AddMass) const
{
	Chaos::PMatrix<FSolverReal, 3, 3> FinalHessian = Chaos::PMatrix<FSolverReal, 3, 3>::Zero;
	if (AddMass)
	{
		for (int32 Alpha = 0; Alpha < 3; Alpha++)
		{
			FinalHessian.SetAt(Alpha, Alpha, Particles.M(ParticleIdx));
		}
	}

	for (int32 IncidentIdx = 0; IncidentIdx < IncidentElements[IncidentIndex].Num(); IncidentIdx++)
	{
		const int32 LocalIndex = IncidentElementsLocal[IncidentIndex][IncidentIdx];
		const int32 ElementIndex = IncidentElements[IncidentIndex][IncidentIdx];
		const Chaos::PMatrix<FSolverReal, 3, 3> DmInv = Base::ElementDmInv(ElementIndex);
		const Chaos::PMatrix<FSolverReal, 3, 3> Fe = Base::F(ElementIndex, Particles);

		const FSolverReal Coeff = Dt * Dt * Measure[ElementIndex];

		ComputeHessianHelper(Fe, DmInv, MuElementArray[ElementIndex], LambdaElementArray[ElementIndex], LocalIndex, Coeff, FinalHessian);
	}
	return FinalHessian;
}

void FGaussSeidelCorotatedTetrahedralConstraints::AddHyperelasticResidualAndHessian(const FSolverParticlesRange& Particles, const int32 ElementIndex, const int32 ElementIndexLocal, const FSolverReal Dt, TVec3<FSolverReal>& ParticleResidual, Chaos::PMatrix<FSolverReal, 3, 3>& ParticleHessian)
{
	const Chaos::PMatrix<FSolverReal, 3, 3> DmInvT = Base::ElementDmInv(ElementIndex).GetTransposed();
	const Chaos::PMatrix<FSolverReal, 3, 3> Fe = F(ElementIndex, Particles);
	Chaos::PMatrix<FSolverReal, 3, 3> P((FSolverReal)0.);

	this->ComputeStress(Fe, MuElementArray[ElementIndex], LambdaElementArray[ElementIndex], P);

	const Chaos::PMatrix<FSolverReal, 3, 3> ForceTerm = -Measure[ElementIndex] * DmInvT * P;
	Chaos::TVector<FSolverReal, 3> Dx((FSolverReal)0.);
	if (ElementIndexLocal > 0)
	{
		for (int32 Alpha = 0; Alpha < 3; Alpha++)
		{
			Dx[Alpha] += ForceTerm.GetAt(Alpha, ElementIndexLocal - 1);
		}
	}
	else
	{
		for (int32 Alpha = 0; Alpha < 3; Alpha++)
		{
			for (int32 Col = 0; Col < 3; Col++)
			{
				Dx[Alpha] -= ForceTerm.GetAt(Alpha, Col);
			}
		}
	}

	Dx *= Dt * Dt;

	for (int32 Alpha = 0; Alpha < 3; Alpha++)
	{
		ParticleResidual[Alpha] -= Dx[Alpha];
	}

	ComputeHessianHelper(Fe, Base::ElementDmInv(ElementIndex), MuElementArray[ElementIndex], LambdaElementArray[ElementIndex], ElementIndexLocal, Dt * Dt * Measure[ElementIndex], ParticleHessian);
}

void FGaussSeidelCorotatedTetrahedralConstraints::InitColor(const FSolverParticlesRange& Particles)
{
	ParticlesPerColor = ComputeNodalColoring(MeshConstraints, Particles, ParticleStartIndex, ParticleEndIndex, IncidentElements, IncidentElementsLocal);
}

void FGaussSeidelCorotatedTetrahedralConstraints::InitColor(const FSolverParticles& Particles)
{
	InitColor(FSolverParticlesRange(Particles));
}

void FGaussSeidelCorotatedTetrahedralConstraints::InitializeCorotatedLambdas()
{
	ComputeStress = [](const Chaos::PMatrix<FSolverReal, 3, 3>& Fe, const FSolverReal InMu, const FSolverReal InLambda, Chaos::PMatrix<FSolverReal, 3, 3>& P)
	{
		PCorotated(Fe, InMu, InLambda, P);
	};
	ComputeHessianHelper = [](const Chaos::PMatrix<FSolverReal, 3, 3>& Fe, const Chaos::PMatrix<FSolverReal, 3, 3>& DmInv, const FSolverReal InMu, const FSolverReal InLambda, const int32 LocalIndex, const FSolverReal Coeff, Chaos::PMatrix<FSolverReal, 3, 3>& FinalHessian)
	{
		Chaos::PMatrix<FSolverReal, 3, 3> JFinvT((FSolverReal)0.);
		JFinvT.SetAt(0, 0, Fe.GetAt(1, 1) * Fe.GetAt(2, 2) - Fe.GetAt(2, 1) * Fe.GetAt(1, 2));
		JFinvT.SetAt(0, 1, Fe.GetAt(2, 0) * Fe.GetAt(1, 2) - Fe.GetAt(1, 0) * Fe.GetAt(2, 2));
		JFinvT.SetAt(0, 2, Fe.GetAt(1, 0) * Fe.GetAt(2, 1) - Fe.GetAt(2, 0) * Fe.GetAt(1, 1));
		JFinvT.SetAt(1, 0, Fe.GetAt(2, 1) * Fe.GetAt(0, 2) - Fe.GetAt(0, 1) * Fe.GetAt(2, 2));
		JFinvT.SetAt(1, 1, Fe.GetAt(0, 0) * Fe.GetAt(2, 2) - Fe.GetAt(2, 0) * Fe.GetAt(0, 2));
		JFinvT.SetAt(1, 2, Fe.GetAt(2, 0) * Fe.GetAt(0, 1) - Fe.GetAt(0, 0) * Fe.GetAt(2, 1));
		JFinvT.SetAt(2, 0, Fe.GetAt(0, 1) * Fe.GetAt(1, 2) - Fe.GetAt(1, 1) * Fe.GetAt(0, 2));
		JFinvT.SetAt(2, 1, Fe.GetAt(1, 0) * Fe.GetAt(0, 2) - Fe.GetAt(0, 0) * Fe.GetAt(1, 2));
		JFinvT.SetAt(2, 2, Fe.GetAt(0, 0) * Fe.GetAt(1, 1) - Fe.GetAt(1, 0) * Fe.GetAt(0, 1));

		const Chaos::PMatrix<FSolverReal, 3, 3> JFinv = JFinvT.GetTransposed();

		if (LocalIndex == 0)
		{
			FSolverReal DmInvSum = FSolverReal(0);
			for (int32 Col = 0; Col < 3; Col++)
			{
				FSolverReal LocalDmSum = FSolverReal(0);
				for (int32 Row = 0; Row < 3; Row++)
				{
					LocalDmSum += DmInv.GetAt(Row, Col);
				}
				DmInvSum += LocalDmSum * LocalDmSum;
			}
			for (int32 Alpha = 0; Alpha < 3; Alpha++)
			{
				FinalHessian.SetAt(Alpha, Alpha, FinalHessian.GetAt(Alpha, Alpha) + Coeff * FSolverReal(2) * InMu * DmInvSum);
			}

			const Chaos::PMatrix<FSolverReal, 3, 3> DmInvJFinv = JFinv * DmInv;
			Chaos::TVector<FSolverReal, 3> L((FSolverReal)0.);
			for (int32 Alpha = 0; Alpha < 3; Alpha++)
			{
				for (int32 Row = 0; Row < 3; Row++)
				{
					L[Alpha] += DmInvJFinv.GetAt(Row, Alpha);
				}
			}
			for (int32 Alpha = 0; Alpha < 3; Alpha++)
			{
				FinalHessian.SetRow(Alpha, FinalHessian.GetRow(Alpha) + Coeff * InLambda * L[Alpha] * L);
			}
		}
		else
		{
			FSolverReal DmInvSum = FSolverReal(0);
			for (int32 Col = 0; Col < 3; Col++)
			{
				DmInvSum += DmInv.GetAt(LocalIndex - 1, Col) * DmInv.GetAt(LocalIndex - 1, Col);
			}
			for (int32 Alpha = 0; Alpha < 3; Alpha++)
			{
				FinalHessian.SetAt(Alpha, Alpha, FinalHessian.GetAt(Alpha, Alpha) + Coeff * FSolverReal(2) * InMu * DmInvSum);
			}

			const Chaos::PMatrix<FSolverReal, 3, 3> DmInvJFinv = JFinv * DmInv;
			Chaos::TVector<FSolverReal, 3> L((FSolverReal)0.);
			for (int32 Alpha = 0; Alpha < 3; Alpha++)
			{
				L[Alpha] = DmInvJFinv.GetAt(LocalIndex - 1, Alpha);
			}
			for (int32 Alpha = 0; Alpha < 3; Alpha++)
			{
				FinalHessian.SetRow(Alpha, FinalHessian.GetRow(Alpha) + Coeff * InLambda * L[Alpha] * L);
			}
		}
	};

	AddAdditionalRes = [](const FSolverParticlesRange& InParticles, const int32 ParticleIdx, const FSolverReal Dt, TVec3<FSolverReal>& Residual) {};
	AddAdditionalHessian = [](const FSolverParticlesRange& InParticles, const int32 ParticleIdx, const FSolverReal Dt, Chaos::PMatrix<FSolverReal, 3, 3>& Hessian) {};
}

TVector<FSolverReal, 3> FGaussSeidelCorotatedTetrahedralConstraints::ComputeDeltax(const int32 ParticleIdx, const int32 IncidentIndex, const FSolverParticlesRange& Particles,
	const FSolverReal Dt) const
{
	Chaos::TVector<FSolverReal, 3> ParticleResidual = ComputePerParticleResidual(ParticleIdx, ParticleIdx - ParticleStartIndex, Particles, Dt, !bDoQuasistatics);

	AddAdditionalRes(Particles, ParticleIdx, Dt, ParticleResidual);

	if (ParticleResidual.Size() > LocalNewtonTol)
	{
		Chaos::PMatrix<FSolverReal, 3, 3> SimplifiedHessian = ComputePerParticleCorotatedHessianSimple(ParticleIdx, ParticleIdx - ParticleStartIndex, Particles, Dt, !bDoQuasistatics);
		AddAdditionalHessian(Particles, ParticleIdx, Dt, SimplifiedHessian);

		const FSolverReal HessianDet = SimplifiedHessian.Determinant();
		if (HessianDet > UE_SMALL_NUMBER)
		{
			Chaos::PMatrix<FSolverReal, 3, 3> HessianInv = SimplifiedHessian.SymmetricCofactorMatrix();
			HessianInv *= FSolverReal(1) / HessianDet;
			return HessianInv.GetTransposed() * (-ParticleResidual);
		}
	}
	return TVector<FSolverReal, 3>((FSolverReal)0.);
}

}// End namespace Chaos::Softs
