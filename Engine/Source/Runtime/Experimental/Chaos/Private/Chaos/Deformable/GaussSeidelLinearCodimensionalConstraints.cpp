// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/Deformable/GaussSeidelLinearCodimensionalConstraints.h"

namespace Chaos::Softs
{

FGaussSeidelLinearTriangleConstraints::FGaussSeidelLinearTriangleConstraints(
	const FSolverParticles& InParticles,
	const TArray<TVector<int32, 3>>& InMesh,
	const FSolverReal& EMesh,
	const FSolverReal& NuMesh)
	: MeshConstraints(InMesh)
{
	Measure.Init((FSolverReal)0., MeshConstraints.Num());
	Lambda = EMesh * NuMesh / (((FSolverReal)1. + NuMesh) * ((FSolverReal)1. - (FSolverReal)2. * NuMesh));
	Mu = EMesh / ((FSolverReal)2. * ((FSolverReal)1. + NuMesh));

	MuElementArray.Init(Mu, MeshConstraints.Num());
	LambdaElementArray.Init(Lambda, MeshConstraints.Num());

	InitializeCodimensionData(FSolverParticlesRange(InParticles));
}

FGaussSeidelLinearTriangleConstraints::FGaussSeidelLinearTriangleConstraints(
	const FSolverParticles& InParticles,
	const TArray<TVector<int32, 3>>& InMesh,
	const TArray<FSolverReal>& EMeshArray,
	const FSolverReal& NuMesh)
	: MeshConstraints(InMesh)
{
	Measure.Init((FSolverReal)0., MeshConstraints.Num());

	InitializeCodimensionData(FSolverParticlesRange(InParticles));
	LambdaElementArray.Init((FSolverReal)0., MeshConstraints.Num());
	MuElementArray.Init((FSolverReal)0., MeshConstraints.Num());

	for (int32 ElemIdx = 0; ElemIdx < InMesh.Num(); ElemIdx++)
	{
		LambdaElementArray[ElemIdx] = EMeshArray[ElemIdx] * NuMesh / (((FSolverReal)1. + NuMesh) * ((FSolverReal)1. - (FSolverReal)2. * NuMesh));
		MuElementArray[ElemIdx] = EMeshArray[ElemIdx] / ((FSolverReal)2. * ((FSolverReal)1. + NuMesh));
	}
}

PMatrix<FSolverReal, 3, 2> FGaussSeidelLinearTriangleConstraints::Ds(const int32 ElemIdx, const FSolverParticlesRange& InParticles) const
{
	if (INDEX_NONE < ElemIdx && ElemIdx < MeshConstraints.Num()
		&& INDEX_NONE < MeshConstraints[ElemIdx][0] && MeshConstraints[ElemIdx][0] < (int32)InParticles.Size()
		&& INDEX_NONE < MeshConstraints[ElemIdx][1] && MeshConstraints[ElemIdx][1] < (int32)InParticles.Size()
		&& INDEX_NONE < MeshConstraints[ElemIdx][2] && MeshConstraints[ElemIdx][2] < (int32)InParticles.Size())
	{
		const TVec3<FSolverReal> P1P0 = InParticles.P(MeshConstraints[ElemIdx][1]) - InParticles.P(MeshConstraints[ElemIdx][0]);
		const TVec3<FSolverReal> P2P0 = InParticles.P(MeshConstraints[ElemIdx][2]) - InParticles.P(MeshConstraints[ElemIdx][0]);

		return PMatrix<FSolverReal, 3, 2>(
			P1P0[0], P1P0[1], P1P0[2],
			P2P0[0], P2P0[1], P2P0[2]);
	}
	else
	{
		return PMatrix<FSolverReal, 3, 2>(
			(FSolverReal)0., (FSolverReal)0, (FSolverReal)0,
			(FSolverReal)0, (FSolverReal)0, (FSolverReal)0);
	}
}

PMatrix<FSolverReal, 3, 2> FGaussSeidelLinearTriangleConstraints::F(const int32 ElemIdx, const FSolverParticlesRange& InParticles) const
{
	if (INDEX_NONE < ElemIdx && ElemIdx < DmInverse.Num())
	{
		return Ds(ElemIdx, InParticles) * DmInverse[ElemIdx];
	}
	else
	{
		return PMatrix<FSolverReal, 3, 2>(
			(FSolverReal)0., (FSolverReal)0, (FSolverReal)0,
			(FSolverReal)0, (FSolverReal)0, (FSolverReal)0);
	}
}

TArray<TArray<int32>> FGaussSeidelLinearTriangleConstraints::GetConstraintsArray() const
{
	TArray<TArray<int32>> Constraints;
	Constraints.SetNum(MeshConstraints.Num());
	for (int32 ElemIdx = 0; ElemIdx < MeshConstraints.Num(); ElemIdx++)
	{
		Constraints[ElemIdx].SetNum(3);
		for (int32 TriLocalIdx = 0; TriLocalIdx < 3; TriLocalIdx++)
		{
			Constraints[ElemIdx][TriLocalIdx] = MeshConstraints[ElemIdx][TriLocalIdx];
		}
	}
	return Constraints;
}

void FGaussSeidelLinearTriangleConstraints::InitializeCodimensionData(const FSolverParticlesRange& Particles)
{
	Measure.Init((FSolverReal)0., MeshConstraints.Num());
	DmInverse.Init(PMatrix<FSolverReal, 2, 2>(0.f, 0.f, 0.f), MeshConstraints.Num());
	for (int32 ElemIdx = 0; ElemIdx < MeshConstraints.Num(); ElemIdx++)
	{
		if (!ensure(MeshConstraints[ElemIdx][0] > INDEX_NONE && MeshConstraints[ElemIdx][0] < (int32)Particles.Size()
			&& MeshConstraints[ElemIdx][1] > INDEX_NONE && MeshConstraints[ElemIdx][1] < (int32)Particles.Size()
			&& MeshConstraints[ElemIdx][2] > INDEX_NONE && MeshConstraints[ElemIdx][2] < (int32)Particles.Size()))
		{
			continue;
		}

		const TVec3<FSolverReal> X1X0 = Particles.GetX(MeshConstraints[ElemIdx][1]) - Particles.GetX(MeshConstraints[ElemIdx][0]);
		const TVec3<FSolverReal> X2X0 = Particles.GetX(MeshConstraints[ElemIdx][2]) - Particles.GetX(MeshConstraints[ElemIdx][0]);
		PMatrix<FSolverReal, 2, 2> Dm((FSolverReal)0., (FSolverReal)0., (FSolverReal)0.);
		Dm.M[0] = X1X0.Size();
		Dm.M[2] = X1X0.Dot(X2X0) / Dm.M[0];
		Dm.M[3] = Chaos::TVector<FSolverReal, 3>::CrossProduct(X1X0, X2X0).Size() / Dm.M[0];
		Measure[ElemIdx] = Chaos::TVector<FSolverReal, 3>::CrossProduct(X1X0, X2X0).Size() / (FSolverReal)2.;
		ensureMsgf(Measure[ElemIdx] > (FSolverReal)0., TEXT("Degenerate triangle detected"));

		const PMatrix<FSolverReal, 2, 2> DmInv = Dm.Inverse();
		DmInverse[ElemIdx] = DmInv;
	}
}

void FGaussSeidelLinearTriangleConstraints::AddHyperelasticResidualAndHessian(const FSolverParticlesRange& Particles, const int32 ElementIndex, const int32 ElementIndexLocal, const FSolverReal Dt, TVec3<FSolverReal>& ParticleResidual, Chaos::PMatrix<FSolverReal, 3, 3>& ParticleHessian)
{
	if (!ensure(ElementIndex > INDEX_NONE
		&& ElementIndex < DmInverse.Num()
		&& ElementIndex < MuElementArray.Num()
		&& ElementIndex < Measure.Num()
		&& ElementIndex < LambdaElementArray.Num()
		&& ElementIndexLocal > INDEX_NONE
		&& ElementIndexLocal < 3))
	{
		return;
	}

	const Chaos::PMatrix<FSolverReal, 2, 2> DmInvT = DmInverse[ElementIndex].GetTransposed(), DmInv = DmInverse[ElementIndex];
	const Chaos::PMatrix<FSolverReal, 3, 2> Fe = F(ElementIndex, Particles);

	PMatrix<FSolverReal, 3, 2> Pe(TVec3<FSolverReal>((FSolverReal)0.), TVec3<FSolverReal>((FSolverReal)0.)), ForceTerm(TVec3<FSolverReal>((FSolverReal)0.), TVec3<FSolverReal>((FSolverReal)0.));
	Pe = FSolverReal(2) * MuElementArray[ElementIndex] * Fe;

	ForceTerm = -Measure[ElementIndex] * Pe * DmInverse[ElementIndex].GetTransposed();

	Chaos::TVector<FSolverReal, 3> Dx((FSolverReal)0.);
	if (ElementIndexLocal > 0)
	{
		for (int32 Alpha = 0; Alpha < 3; Alpha++)
		{
			Dx[Alpha] += ForceTerm.M[ElementIndexLocal * 3 - 3 + Alpha];
		}
	}
	else
	{
		for (int32 Alpha = 0; Alpha < 3; Alpha++)
		{
			for (int32 Col = 0; Col < 2; Col++)
			{
				Dx[Alpha] -= ForceTerm.M[Col * 3 + Alpha];
			}
		}
	}

	Dx *= Dt * Dt;
	ParticleResidual -= Dx;
	const FSolverReal Coeff = Dt * Dt * Measure[ElementIndex];

	if (ElementIndexLocal == 0)
	{
		FSolverReal DmInvSum = FSolverReal(0);
		for (int32 Row = 0; Row < 2; Row++)
		{
			FSolverReal LocalDmSum = FSolverReal(0);
			for (int32 Col = 0; Col < 2; Col++)
			{
				LocalDmSum += DmInv.M[Row * 2 + Col];
			}
			DmInvSum += LocalDmSum * LocalDmSum;
		}
		for (int32 Alpha = 0; Alpha < 3; Alpha++)
		{
			ParticleHessian.SetAt(Alpha, Alpha, ParticleHessian.GetAt(Alpha, Alpha) + Coeff * FSolverReal(2) * MuElementArray[ElementIndex] * DmInvSum);
		}
	}
	else
	{
		FSolverReal DmInvSum = FSolverReal(0);
		for (int32 Col = 0; Col < 2; Col++)
		{
			DmInvSum += DmInv.GetAt(ElementIndexLocal - 1, Col) * DmInv.GetAt(ElementIndexLocal - 1, Col);
		}
		for (int32 Alpha = 0; Alpha < 3; Alpha++)
		{
			ParticleHessian.SetAt(Alpha, Alpha, ParticleHessian.GetAt(Alpha, Alpha) + Dt * Dt * Measure[ElementIndex] * FSolverReal(2) * MuElementArray[ElementIndex] * DmInvSum);
		}
	}
}

}  // End namespace Chaos::Softs
