// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/Deformable/GaussSeidelMainConstraint.h"
#include "Chaos/Math/Krylov.h"
#include "ChaosLog.h"

DEFINE_LOG_CATEGORY(LogDeformableGaussSeidelMainConstraint);

namespace Chaos::Softs
{
	int32 MaxItCG = 50;

	FAutoConsoleVariableRef CVarClothMaxItCG(TEXT("p.Chaos.Cloth.MaxItCG"), MaxItCG, TEXT("Max iter for CG [def: 50]"));

	FSolverReal CGTol = 1e-4f;

	FAutoConsoleVariableRef CVarClothCGTol(TEXT("p.Chaos.Cloth.CGTol"), CGTol, TEXT("CG Tolerance [def: 1e-4]"));


	void FGaussSeidelMainConstraints::AddStaticConstraints(const TArray<TArray<int32>>& ExtraConstraints, const TArray<TArray<int32>>& ExtraIncidentElements, const TArray<TArray<int32>>& ExtraIncidentElementsLocal)
	{
		const TArray<TArray<int32>>* IncidentElementsPtr = &ExtraIncidentElements;
		const TArray<TArray<int32>>* IncidentElementsLocalPtr = &ExtraIncidentElementsLocal;
		TArray<TArray<int32>> RecomputedIncidentElements, RecomputedIncidentElementsLocal;
		if (!IsClean(ExtraConstraints, ExtraIncidentElements, ExtraIncidentElementsLocal))
		{
			RecomputedIncidentElements = Chaos::Utilities::ComputeIncidentElements(ExtraConstraints, &RecomputedIncidentElementsLocal);
			IncidentElementsPtr = &RecomputedIncidentElements;
			IncidentElementsLocalPtr = &RecomputedIncidentElementsLocal;
		}

		int32 Offset = StaticConstraints.Num();
		StaticConstraints += ExtraConstraints;
		for (int32 ParticleIdx = 0; ParticleIdx < IncidentElementsPtr->Num(); ParticleIdx++)
		{
			if ((*IncidentElementsPtr)[ParticleIdx].Num() > 0)
			{
				TArray<int32> ExtraIncidentElementsWithOffset = (*IncidentElementsPtr)[ParticleIdx];
				for (int32 LocalIdx = 0; LocalIdx < ExtraIncidentElementsWithOffset.Num(); LocalIdx++)
				{
					ExtraIncidentElementsWithOffset[LocalIdx] += Offset;
				}
				StaticIncidentElements[ParticleIdx] += ExtraIncidentElementsWithOffset;
				StaticIncidentElementsLocal[ParticleIdx] += (*IncidentElementsLocalPtr)[ParticleIdx];
			}
		}
		if (StaticIncidentElementsOffsets.Num() > 0)
		{
			StaticIncidentElementsOffsets.RemoveAt(StaticIncidentElementsOffsets.Num() - 1);
		}
		StaticIncidentElementsOffsets.Add(Offset);
		StaticIncidentElementsOffsets.Add(StaticConstraints.Num());
	}

	void FGaussSeidelMainConstraints::AddTransientConstraints(const TArray<TArray<int32>>& ExtraConstraints, const TArray<TArray<int32>>& ExtraIncidentElements, const TArray<TArray<int32>>& ExtraIncidentElementsLocal, bool CheckIncidentElements)
	{
		const TArray<TArray<int32>>* IncidentElementsPtr = &ExtraIncidentElements;
		const TArray<TArray<int32>>* IncidentElementsLocalPtr = &ExtraIncidentElementsLocal;
		TArray<TArray<int32>> RecomputedIncidentElements, RecomputedIncidentElementsLocal;
		if (CheckIncidentElements && !IsClean(ExtraConstraints, ExtraIncidentElements, ExtraIncidentElementsLocal))
		{
			RecomputedIncidentElements = Chaos::Utilities::ComputeIncidentElements(ExtraConstraints, &RecomputedIncidentElementsLocal);
			IncidentElementsPtr = &RecomputedIncidentElements;
			IncidentElementsLocalPtr = &RecomputedIncidentElementsLocal;
		}

		int32 Offset = TransientConstraints.Num();
		TransientConstraints += ExtraConstraints;
		for (int32 ParticleIdx = 0; ParticleIdx < IncidentElementsPtr->Num(); ParticleIdx++)
		{
			if ((*IncidentElementsPtr)[ParticleIdx].Num() > 0)
			{
				TArray<int32> ExtraIncidentElementsWithOffset = (*IncidentElementsPtr)[ParticleIdx];
				for (int32 LocalIdx = 0; LocalIdx < ExtraIncidentElementsWithOffset.Num(); LocalIdx++)
				{
					ExtraIncidentElementsWithOffset[LocalIdx] += Offset;
				}
				TransientIncidentElements[ParticleIdx] += ExtraIncidentElementsWithOffset;
				TransientIncidentElementsLocal[ParticleIdx] += (*IncidentElementsLocalPtr)[ParticleIdx];
			}
		}
		if (TransientIncidentElementsOffsets.Num() > 0)
		{
			TransientIncidentElementsOffsets.RemoveAt(TransientIncidentElementsOffsets.Num() - 1);
		}
		TransientIncidentElementsOffsets.Add(Offset);
		TransientIncidentElementsOffsets.Add(TransientConstraints.Num());
	}

	void FGaussSeidelMainConstraints::AddDynamicConstraints(const TArray<TArray<int32>>& ExtraConstraints, const TArray<TArray<int32>>& ExtraIncidentElements, const TArray<TArray<int32>>& ExtraIncidentElementsLocal, bool CheckIncidentElements)
	{
		const TArray<TArray<int32>>* IncidentElementsPtr = &ExtraIncidentElements;
		const TArray<TArray<int32>>* IncidentElementsLocalPtr = &ExtraIncidentElementsLocal;
		TArray<TArray<int32>> RecomputedIncidentElements, RecomputedIncidentElementsLocal;
		if (CheckIncidentElements && !IsClean(ExtraConstraints, ExtraIncidentElements, ExtraIncidentElementsLocal))
		{
			RecomputedIncidentElements = Chaos::Utilities::ComputeIncidentElements(ExtraConstraints, &RecomputedIncidentElementsLocal);
			IncidentElementsPtr = &RecomputedIncidentElements;
			IncidentElementsLocalPtr = &RecomputedIncidentElementsLocal;
		}

		int32 Offset = DynamicConstraints.Num();
		DynamicConstraints += ExtraConstraints;
		for (int32 ParticleIdx = 0; ParticleIdx < IncidentElementsPtr->Num(); ParticleIdx++)
		{
			if ((*IncidentElementsPtr)[ParticleIdx].Num() > 0)
			{
				TArray<int32> ExtraIncidentElementsWithOffset = (*IncidentElementsPtr)[ParticleIdx];
				for (int32 LocalIdx = 0; LocalIdx < ExtraIncidentElementsWithOffset.Num(); LocalIdx++)
				{
					ExtraIncidentElementsWithOffset[LocalIdx] += Offset;
				}
				DynamicIncidentElements[ParticleIdx] += ExtraIncidentElementsWithOffset;
				DynamicIncidentElementsLocal[ParticleIdx] += (*IncidentElementsLocalPtr)[ParticleIdx];
			}
		}
		if (DynamicIncidentElementsOffsets.Num() > 0)
		{
			DynamicIncidentElementsOffsets.RemoveAt(DynamicIncidentElementsOffsets.Num() - 1);
		}
		DynamicIncidentElementsOffsets.Add(Offset);
		DynamicIncidentElementsOffsets.Add(DynamicConstraints.Num());
	}

	TArray<TVec3<FSolverReal>> FGaussSeidelMainConstraints::ComputeNewtonResiduals(const FSolverParticlesRange& ParticlesRange, const FSolverReal Dt, const bool Write2File, TArray<PMatrix<FSolverReal, 3, 3>>* AllParticleHessian)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(STAT_ChaosGaussSeidelComputeNewtonResidual);

		TArray<TVec3<FSolverReal>> NewtonResidual;
		NewtonResidual.Init(TVec3<FSolverReal>((FSolverReal)0.), xtilde.Num());

		if (AllParticleHessian)
		{
			AllParticleHessian->Init(PMatrix<FSolverReal, 3, 3>((FSolverReal)0.), ParticlesRange.Size());
		}

		for (int32 ColorIdx = 0; ColorIdx < ParticlesPerColor.Num(); ColorIdx++)
		{
			PhysicsParallelFor(ParticlesPerColor[ColorIdx].Num(), [this, &AllParticleHessian, Dt, ColorIdx, &NewtonResidual, &ParticlesRange](const int32 BatchIdx)
				{
					const int32 ParticleIdx = ParticlesPerColor[ColorIdx][BatchIdx];

					if (ParticlesRange.InvM(ParticleIdx) != (FSolverReal)0.)
					{
						int32 ConstraintIndex = 0;
						Chaos::PMatrix<FSolverReal, 3, 3> ParticleHessian((FSolverReal)0., (FSolverReal)0., (FSolverReal)0.);

						this->ComputeInitialResidualAndHessian(ParticlesRange, ParticleIdx, Dt, NewtonResidual[ParticleIdx], ParticleHessian);

						for (int32 IncidentIdx = 0; IncidentIdx < StaticIncidentElements[ParticleIdx].Num(); IncidentIdx++)
						{
							while (StaticIncidentElements[ParticleIdx][IncidentIdx] >= StaticIncidentElementsOffsets[ConstraintIndex + 1] && ConstraintIndex < StaticIncidentElementsOffsets.Num() - 1)
							{
								ConstraintIndex += 1;
							}

							this->AddStaticConstraintResidualAndHessian[ConstraintIndex](ParticlesRange, StaticIncidentElements[ParticleIdx][IncidentIdx] - StaticIncidentElementsOffsets[ConstraintIndex], StaticIncidentElementsLocal[ParticleIdx][IncidentIdx], Dt, NewtonResidual[ParticleIdx], ParticleHessian);
						}

						ConstraintIndex = 0;

						for (int32 IncidentIdx = 0; IncidentIdx < TransientIncidentElements[ParticleIdx].Num(); IncidentIdx++)
						{
							while (TransientIncidentElements[ParticleIdx][IncidentIdx] >= TransientIncidentElementsOffsets[ConstraintIndex + 1] && ConstraintIndex < TransientIncidentElementsOffsets.Num() - 1)
							{
								ConstraintIndex += 1;
							}

							this->AddTransientConstraintResidualAndHessian[ConstraintIndex](ParticlesRange, TransientIncidentElements[ParticleIdx][IncidentIdx] - TransientIncidentElementsOffsets[ConstraintIndex], TransientIncidentElementsLocal[ParticleIdx][IncidentIdx], Dt, NewtonResidual[ParticleIdx], ParticleHessian);
						}

						if (AllParticleHessian)
						{
							(*AllParticleHessian)[ParticleIdx] = ParticleHessian;
						}
					}
			}, ParticlesPerColor[ColorIdx].Num() < 1000);
		}

		FSolverReal NewtonNorm = (FSolverReal)0.;
		for (int32 ParticleIdx = 0; ParticleIdx < NewtonResidual.Num(); ParticleIdx++)
		{
			NewtonNorm += NewtonResidual[ParticleIdx].SizeSquared();
		}

		NewtonNorm = FMath::Sqrt(NewtonNorm);

		UE_LOGF(LogChaos, Display, "Current Iteration: %d", CurrentIt);

		UE_LOGF(LogChaos, Display, "Newton Residual is %f", NewtonNorm);

		PassedIters++;


#if WITH_EDITOR
		if (Write2File)
		{
			FString File = FPaths::ProjectDir();
			File.Append(TEXT("/DebugOutput/NewtonResidual.txt"));
			if (PassedIters == 0)
			{
				FFileHelper::SaveStringToFile(FString("Newton Norm\r\n"), *File);
			}
			FFileHelper::SaveStringToFile((FString::SanitizeFloat(NewtonNorm) + FString(",\r\n")), *File, FFileHelper::EEncodingOptions::AutoDetect, &IFileManager::Get(), EFileWrite::FILEWRITE_Append);
		}
#endif

		return NewtonResidual;

	}

	TArray<TVec3<FSolverReal>> FGaussSeidelMainConstraints::ComputeNewtonResiduals(const FSolverParticles& Particles, const FSolverReal Dt, const bool Write2File, TArray<PMatrix<FSolverReal, 3, 3>>* AllParticleHessian)
	{
		return ComputeNewtonResiduals(FSolverParticlesRange(Particles), Dt, Write2File, AllParticleHessian);
	}

	void FGaussSeidelMainConstraints::ApplyCG(FSolverParticlesRange& ParticlesRange, const FSolverReal Dt)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(STAT_ChaosGaussSeidelApplyCG);
		TFunction<void(TArray<FSolverVec3>&)> ProjectBCs = [&ParticlesRange](TArray<FSolverVec3>& Y)
		{
			for (int32 ParticleIdx = 0; ParticleIdx < Y.Num(); ParticleIdx++)
			{
				if (ParticlesRange.InvM(ParticleIdx) == 0.f)
				{
					Y[ParticleIdx] = FSolverVec3(0.f);
				}
			}
		};

		const int32 MaxNewtonIt = 1;

		for (int32 It = 0; It < MaxNewtonIt; It++)
		{
			const TArray<FSolverVec3> Residual = ComputeNewtonResiduals(ParticlesRange, Dt, false, nullptr);

			auto Multiply = [this, &ProjectBCs, &Dt, &ParticlesRange](TArray<FSolverVec3>& Y, const TArray<FSolverVec3>& X)
			{
				TArray<FSolverVec3> XProj = X;
				ProjectBCs(XProj);

				Y.Init(FSolverVec3((FSolverReal)0.), XProj.Num());

				for (int32 ParticleIdx = 0; ParticleIdx < this->AddInternalForceDifferentials.Num(); ParticleIdx++)
				{
					this->AddInternalForceDifferentials[ParticleIdx](ParticlesRange, XProj, Y);
				}

				PhysicsParallelFor(Y.Num(),
					[&Y, &ParticlesRange, &XProj, Dt](const int32 ParticleIdx)
					{
						Y[ParticleIdx] = ParticlesRange.M(ParticleIdx) * XProj[ParticleIdx] + Dt * Dt * Y[ParticleIdx];
					});

				ProjectBCs(Y);
			};

			TArray<FSolverVec3> DeltaX;
			DeltaX.Init(FSolverVec3(0.f), ParticlesRange.Size());

			Chaos::LanczosCG<FSolverReal>(Multiply, DeltaX, Residual, MaxItCG, CGTol, use_list.Get());

			PhysicsParallelFor(ParticlesRange.Size(), [&ParticlesRange, &DeltaX](const int32 ParticleIdx)
				{
					ParticlesRange.P(ParticleIdx) -= DeltaX[ParticleIdx];
				});
		}
	}

	void FGaussSeidelMainConstraints::ApplyCG(FSolverParticles& Particles, const FSolverReal Dt)
	{
		FSolverParticlesRange Range(Particles);
		ApplyCG(Range, Dt);
	}

	// -------------------------------------------------------------------
	// Methods moved from header
	// -------------------------------------------------------------------

	FGaussSeidelMainConstraints::FGaussSeidelMainConstraints(
		const FSolverParticlesRange& InParticles,
		const bool bDoQuasistaticsIn,
		const bool bDoSORIn,
		const FSolverReal InOmegaSOR,
		const int32 ParallelMaxIn,
		const FSolverReal MaxDxRatioIn,
		const FDeformableXPBDCorotatedParams& InParams)
		: bDoQuasistatics(bDoQuasistaticsIn)
		, bDoAcceleration(bDoSORIn)
		, OmegaSOR(InOmegaSOR)
		, ParallelMax(ParallelMaxIn)
		, CorotatedParams(InParams)
	{
		Resize((int32)InParticles.Size());

		InitializeLambdas();

		TVec3<FSolverReal> MaxCoord((FSolverReal)100.), MinCoord((FSolverReal)-100.);
		for (int32 ParticleIdx = 0; ParticleIdx < (int32)InParticles.Size(); ParticleIdx++)
		{
			for (int32 Alpha = 0; Alpha < 3; Alpha++)
			{
				if (InParticles.X(ParticleIdx)[Alpha] < MinCoord[Alpha])
				{
					MinCoord[Alpha] = InParticles.X(ParticleIdx)[Alpha];
				}
				if (InParticles.X(ParticleIdx)[Alpha] > MaxCoord[Alpha])
				{
					MaxCoord[Alpha] = InParticles.X(ParticleIdx)[Alpha];
				}
			}
		}
		MaxDxSize = (MaxCoord - MinCoord).Size() * MaxDxRatioIn;
	}

	FGaussSeidelMainConstraints::FGaussSeidelMainConstraints(
		const FSolverParticles& InParticles,
		const bool bDoQuasistaticsIn,
		const bool bDoSORIn,
		const FSolverReal InOmegaSOR,
		const int32 ParallelMaxIn,
		const FSolverReal MaxDxRatioIn,
		const FDeformableXPBDCorotatedParams& InParams)
		: FGaussSeidelMainConstraints(
			FSolverParticlesRange(InParticles),
			bDoQuasistaticsIn, bDoSORIn, InOmegaSOR, ParallelMaxIn, MaxDxRatioIn, InParams)
	{}

	FGaussSeidelMainConstraints::FGaussSeidelMainConstraints(const FSolverParticlesRange& InParticles, const FCollectionPropertyConstFacade& Property, const FSolverReal MaxDxRatioIn)
		: bDoQuasistatics(GetDoQuasistatics(Property, false))
		, bDoAcceleration(GetAccelerateSolverUsingSOR(Property, true))
		, OmegaSOR(GetOmegaSOR(Property, (FSolverReal)1.6))
		, ParallelMax(1000)
		, CorotatedParams(FDeformableXPBDCorotatedParams())
	{
		Resize(InParticles.Size());
		InitializeLambdas();
		TVec3<FSolverReal> MaxCoord((FSolverReal)100.), MinCoord((FSolverReal)-100.);
		for (int32 Vdx = 0; Vdx < (int32)InParticles.Size(); ++Vdx)
		{
			MaxCoord.ComponentwiseMax(InParticles.X(Vdx));
			MinCoord.ComponentwiseMin(InParticles.X(Vdx));
		}
		MaxDxSize = (MaxCoord - MinCoord).Size() * MaxDxRatioIn;
	}

	FGaussSeidelMainConstraints::FGaussSeidelMainConstraints(const FSolverParticles& InParticles, const FCollectionPropertyConstFacade& Property, const FSolverReal MaxDxRatioIn)
		: FGaussSeidelMainConstraints(
			FSolverParticlesRange(InParticles),
			Property, MaxDxRatioIn)
	{}

	void FGaussSeidelMainConstraints::Resize(const int32 NewSize)
	{
		xtilde.SetNum(NewSize);
		StaticIncidentElements.SetNum(NewSize);
		StaticIncidentElementsLocal.SetNum(NewSize);
		DynamicIncidentElements.SetNum(NewSize);
		DynamicIncidentElementsLocal.SetNum(NewSize);
		TransientIncidentElements.SetNum(NewSize);
		TransientIncidentElementsLocal.SetNum(NewSize);
		X_k_1.Init(TVector<FSolverReal, 3>(FSolverReal(0.)), NewSize);
		X_k.Init(TVector<FSolverReal, 3>(FSolverReal(0.)), NewSize);
	}

	int32 FGaussSeidelMainConstraints::AddStaticConstraintResidualAndHessianRange(int32 NumConstraints)
	{
		int32 CurrentSize = AddStaticConstraintResidualAndHessian.Num();
		AddStaticConstraintResidualAndHessian.AddDefaulted(NumConstraints);
		return CurrentSize;
	}

	int32 FGaussSeidelMainConstraints::AddTransientConstraintResidualAndHessianRange(int32 NumConstraints)
	{
		int32 CurrentSize = AddTransientConstraintResidualAndHessian.Num();
		AddTransientConstraintResidualAndHessian.AddDefaulted(NumConstraints);
		return CurrentSize;
	}

	int32 FGaussSeidelMainConstraints::AddDynamicConstraintResidualAndHessianRange(int32 NumConstraints)
	{
		int32 CurrentSize = AddDynamicConstraintResidualAndHessian.Num();
		AddDynamicConstraintResidualAndHessian.AddDefaulted(NumConstraints);
		return CurrentSize;
	}

	int32 FGaussSeidelMainConstraints::AddCrossBodyStaticConstraintResidualAndHessianRange(int32 NumConstraints)
	{
		int32 CurrentSize = AddCrossBodyStaticConstraintResidualAndHessian.Num();
		AddCrossBodyStaticConstraintResidualAndHessian.AddDefaulted(NumConstraints);
		return CurrentSize;
	}

	int32 FGaussSeidelMainConstraints::AddCrossBodyTransientConstraintResidualAndHessianRange(int32 NumConstraints)
	{
		int32 CurrentSize = AddCrossBodyTransientConstraintResidualAndHessian.Num();
		AddCrossBodyTransientConstraintResidualAndHessian.AddDefaulted(NumConstraints);
		return CurrentSize;
	}

	int32 FGaussSeidelMainConstraints::AddPerNodeHessianRange(int32 NumConstraints)
	{
		int32 CurrentSize = AddPerNodeHessian.Num();
		AddPerNodeHessian.AddDefaulted(NumConstraints);
		return CurrentSize;
	}

	int32 FGaussSeidelMainConstraints::AddTransientPerNodeHessianRange(int32 NumConstraints)
	{
		int32 CurrentSize = AddTransientPerNodeHessian.Num();
		AddTransientPerNodeHessian.AddDefaulted(NumConstraints);
		return CurrentSize;
	}

	int32 FGaussSeidelMainConstraints::AddAddInternalForceDifferentialsRange(int32 NumConstraints)
	{
		int32 CurrentSize = AddInternalForceDifferentials.Num();
		AddInternalForceDifferentials.AddDefaulted(NumConstraints);
		return CurrentSize;
	}

	void FGaussSeidelMainConstraints::ResetDynamicConstraints()
	{
		DynamicConstraints = {};
		for (int32 ParticleIdx = 0; ParticleIdx < DynamicIncidentElements.Num(); ParticleIdx++)
		{
			DynamicIncidentElements[ParticleIdx].SetNum(0);
			DynamicIncidentElementsLocal[ParticleIdx].SetNum(0);
		}
		DynamicIncidentElementsOffsets = {};
	}

	void FGaussSeidelMainConstraints::ResetTransientConstraints()
	{
		TransientConstraints = {};
		for (int32 ParticleIdx = 0; ParticleIdx < TransientIncidentElements.Num(); ParticleIdx++)
		{
			TransientIncidentElements[ParticleIdx].SetNum(0);
			TransientIncidentElementsLocal[ParticleIdx].SetNum(0);
		}
		TransientIncidentElementsOffsets = {};
	}

	void FGaussSeidelMainConstraints::ResetTransientConstraintsAndRules()
	{
		ResetTransientConstraints();
		AddTransientConstraintResidualAndHessian.Empty();
		AddTransientPerNodeHessian.Empty();
	}

	void FGaussSeidelMainConstraints::Apply(FSolverParticles& Particles, const FSolverReal Dt, const int32 MaxWriteIters, const bool Write2File, const TPBDActiveView<FSolverParticles>* InParticleActiveView)
	{
		FSolverParticlesRange ParticlesRange(Particles);
		Apply(ParticlesRange, Dt, MaxWriteIters, Write2File, InParticleActiveView);
	}

	void FGaussSeidelMainConstraints::Apply(FSolverParticlesRange& ParticlesRange, const FSolverReal Dt, const int32 MaxWriteIters, const bool Write2File, const TPBDActiveView<FSolverParticles>* InParticleActiveView)
	{
		PERF_SCOPE(STAT_ChaosGSMainConstraint_Apply);

		if (DebugResidual && PassedIters < MaxWriteIters)
		{
			ComputeNewtonResiduals(ParticlesRange, Dt, Write2File);
		}

		std::atomic<int32> ParticleFailureCounter(0);

		for (int32 ColorIdx = 0; ColorIdx < ParticlesPerColor.Num(); ColorIdx++)
		{
			int32 NumBatch = ParticlesPerColor[ColorIdx].Num() / CorotatedParams.XPBDCorotatedBatchSize;
			if (ParticlesPerColor[ColorIdx].Num() % CorotatedParams.XPBDCorotatedBatchSize != 0)
			{
				NumBatch ++;
			}

			PhysicsParallelFor(NumBatch, [&](const int32 BatchIndex)
				{
					for (int32 BatchSubIndex = 0; BatchSubIndex < CorotatedParams.XPBDCorotatedBatchSize; BatchSubIndex++) {
						const int32 TaskIndex = CorotatedParams.XPBDCorotatedBatchSize * BatchIndex + BatchSubIndex;
						if (TaskIndex < ParticlesPerColor[ColorIdx].Num())
						{
							if (ParticlesRange.InvM(ParticlesPerColor[ColorIdx][TaskIndex]) != FSolverReal(0))
							{
								const int32 ParticleIndex = ParticlesPerColor[ColorIdx][TaskIndex];
								if (!ApplySingleParticle(ParticleIndex, Dt, ParticlesRange))
								{
									ParticleFailureCounter.fetch_add(1, std::memory_order_relaxed);
								}
							}
						}
					}
				}, NumBatch < CorotatedParams.XPBDCorotatedBatchThreshold);

		}


		if (bDoAcceleration)
		{
			PERF_SCOPE(STAT_ChaosGSMainConstraint_Acceleration);
			if (InParticleActiveView)
			{
				// TODO: Update TPBDActiveView to support FSolverParticlesRange
				InParticleActiveView->ParallelFor(
					[this, &ParticlesRange](FSolverParticles& /*Unused*/, int32 ParticleIndex)
					{
						this->AccelerationTechniquePerParticle(ParticlesRange, ParticleIndex);
					}, CorotatedParams.XPBDCorotatedBatchSize);
			}
			else
			{
				PhysicsParallelFor(ParticlesRange.Size(), [this, &ParticlesRange](const int32 ParticleIndex)
					{
						this->AccelerationTechniquePerParticle(ParticlesRange, ParticleIndex);
					}, ParticlesRange.Size() < 1000);
			}
		}

		CurrentIt ++;

	#if WITH_EDITOR
		if (ParticleFailureCounter.load(std::memory_order_relaxed) > 0)
		{
			UE_LOGF(LogDeformableGaussSeidelMainConstraint, Warning, "%d Particle(s) are skipped because of too large dx size", ParticleFailureCounter.load(std::memory_order_relaxed));
		}
#endif
	}

	void FGaussSeidelMainConstraints::InitStaticColor(const FSolverParticlesRange& ParticlesRange, const TPBDActiveView<FSolverParticles>* InParticleActiveView)
	{
		StaticParticlesPerColor = ComputeNodalColoring(StaticConstraints, ParticlesRange, 0, ParticlesRange.Size(), StaticIncidentElements, StaticIncidentElementsLocal, InParticleActiveView, &StaticParticleColors);
		ParticleColors = StaticParticleColors;
		ParticlesPerColor = StaticParticlesPerColor;
	}

	void FGaussSeidelMainConstraints::InitStaticColor(const FSolverParticles& Particles, const TPBDActiveView<FSolverParticles>* InParticleActiveView)
	{
		InitStaticColor(FSolverParticlesRange(Particles), InParticleActiveView);
	}

	void FGaussSeidelMainConstraints::InitTransientColor(const FSolverParticlesRange& ParticlesRange)
	{
		ensureMsgf(StaticParticleColors.Num() == ParticlesRange.Size(), TEXT("InitStaticColor must be called before InitTransientColor"));
		PERF_SCOPE(STAT_ChaosGSMainConstraint_InitTransientColor);
		ParticleColors = StaticParticleColors;
		ParticlesPerColor = StaticParticlesPerColor;
		Chaos::ComputeExtraNodalColoring(StaticConstraints, DynamicConstraints, TransientConstraints, ParticlesRange, StaticIncidentElements, DynamicIncidentElements, TransientIncidentElements, ParticleColors, ParticlesPerColor);
	}

	void FGaussSeidelMainConstraints::InitTransientColor(const FSolverParticles& Particles)
	{
		InitTransientColor(FSolverParticlesRange(Particles));
	}

	void FGaussSeidelMainConstraints::InitDynamicColor(const FSolverParticlesRange& ParticlesRange)
	{
		ensureMsgf(StaticParticleColors.Num() == ParticlesRange.Size(), TEXT("InitStaticColor must be called before InitDynamicColor"));
		PERF_SCOPE(STAT_ChaosGSMainConstraint_InitDynamicColor);
		ParticleColors = StaticParticleColors;
		ParticlesPerColor = StaticParticlesPerColor;
		Chaos::ComputeExtraNodalColoring(StaticConstraints, DynamicConstraints, ParticlesRange, StaticIncidentElements, TransientIncidentElements, ParticleColors, ParticlesPerColor);
	}

	void FGaussSeidelMainConstraints::InitDynamicColor(const FSolverParticles& Particles)
	{
		InitDynamicColor(FSolverParticlesRange(Particles));
	}

	void FGaussSeidelMainConstraints::Init(const FSolverReal Dt, const FSolverParticlesRange& ParticlesRange)
	{
		Resize((int32)ParticlesRange.Size());

		PERF_SCOPE(STAT_ChaosGSMainConstraint_Init);
		ResetTransientConstraints();
		if (!bDoQuasistatics)
		{
			for (int32 ParticleIdx = 0; ParticleIdx < xtilde.Num(); ParticleIdx++)
			{
				xtilde[ParticleIdx] = ParticlesRange.P(ParticleIdx); // Here P = xn + dt*(vn + dt*an) after evolution initial guess
			}
		}
		CurrentIt = 0;
	}

	void FGaussSeidelMainConstraints::Init(const FSolverReal Dt, const FSolverParticles& Particles)
	{
		Init(Dt, FSolverParticlesRange(Particles));
	}

	bool FGaussSeidelMainConstraints::IsClean(const TArray<TArray<int32>>& ConstraintsIn, const TArray<TArray<int32>>& IncidentElementsIn, const TArray<TArray<int32>>& IncidentElementsLocalIn)
	{
		if (IncidentElementsIn.Num() == IncidentElementsLocalIn.Num())
		{
			int32 TotalEntries = 0;
			for (int32 ParticleIdx = 0; ParticleIdx < IncidentElementsIn.Num(); ParticleIdx++)
			{
				TotalEntries += IncidentElementsIn[ParticleIdx].Num();
				if (IncidentElementsIn[ParticleIdx].Num() != IncidentElementsLocalIn[ParticleIdx].Num())
				{
					return false;
				}
				for (int32 IncidentIdx = 0; IncidentIdx < IncidentElementsIn[ParticleIdx].Num(); IncidentIdx++)
				{
					if (IncidentElementsIn[ParticleIdx][IncidentIdx] >= ConstraintsIn.Num() || IncidentElementsLocalIn[ParticleIdx][IncidentIdx] >= ConstraintsIn[IncidentElementsIn[ParticleIdx][IncidentIdx]].Num())
					{
						return false;
					}
				}
			}
			if (TotalEntries > 0)
			{
				return true;
			}
		}
		return false;
	}

	bool FGaussSeidelMainConstraints::ApplySingleParticle(const int32 ParticleIdx, const FSolverReal Dt, FSolverParticlesRange& Particles)
	{
		int32 ConstraintIndex = 0;
		Chaos::TVector<FSolverReal, 3> ParticleResidual((FSolverReal)0.);
		Chaos::PMatrix<FSolverReal, 3, 3> ParticleHessian((FSolverReal)0., (FSolverReal)0., (FSolverReal)0.);

		ComputeInitialResidualAndHessian(Particles, ParticleIdx, Dt, ParticleResidual, ParticleHessian);

		for (int32 IncidentIdx = 0; IncidentIdx < StaticIncidentElements[ParticleIdx].Num(); IncidentIdx++)
		{
			while (StaticIncidentElements[ParticleIdx][IncidentIdx] >= StaticIncidentElementsOffsets[ConstraintIndex + 1] && ConstraintIndex < StaticIncidentElementsOffsets.Num() - 1)
			{
				ConstraintIndex ++;
			}

			AddStaticConstraintResidualAndHessian[ConstraintIndex](Particles, StaticIncidentElements[ParticleIdx][IncidentIdx] - StaticIncidentElementsOffsets[ConstraintIndex], StaticIncidentElementsLocal[ParticleIdx][IncidentIdx], Dt, ParticleResidual, ParticleHessian);
		}

		ConstraintIndex = 0;

		for (int32 IncidentIdx = 0; IncidentIdx < DynamicIncidentElements[ParticleIdx].Num(); IncidentIdx++)
		{
			while (DynamicIncidentElements[ParticleIdx][IncidentIdx] >= DynamicIncidentElementsOffsets[ConstraintIndex + 1] && ConstraintIndex < DynamicIncidentElementsOffsets.Num() - 1)
			{
				ConstraintIndex ++;
			}

			AddDynamicConstraintResidualAndHessian[ConstraintIndex](Particles, DynamicIncidentElements[ParticleIdx][IncidentIdx] - DynamicIncidentElementsOffsets[ConstraintIndex], DynamicIncidentElementsLocal[ParticleIdx][IncidentIdx], Dt, ParticleResidual, ParticleHessian);
		}

		ConstraintIndex = 0;

		for (int32 IncidentIdx = 0; IncidentIdx < TransientIncidentElements[ParticleIdx].Num(); IncidentIdx++)
		{
			while (TransientIncidentElements[ParticleIdx][IncidentIdx] >= TransientIncidentElementsOffsets[ConstraintIndex + 1] && ConstraintIndex < TransientIncidentElementsOffsets.Num() - 1)
			{
				ConstraintIndex ++;
			}

			AddTransientConstraintResidualAndHessian[ConstraintIndex](Particles, TransientIncidentElements[ParticleIdx][IncidentIdx] - TransientIncidentElementsOffsets[ConstraintIndex], TransientIncidentElementsLocal[ParticleIdx][IncidentIdx], Dt, ParticleResidual, ParticleHessian);
		}

		for (int32 HessianIdx = 0; HessianIdx < AddPerNodeHessian.Num(); HessianIdx++)
		{
			AddPerNodeHessian[HessianIdx](ParticleIdx, Dt, ParticleHessian);
		}
		for (int32 HessianIdx = 0; HessianIdx < AddTransientPerNodeHessian.Num(); HessianIdx++)
		{
			AddTransientPerNodeHessian[HessianIdx](ParticleIdx, Dt, ParticleHessian);
		}
		FSolverReal HessianScale = (FSolverReal)1.;
		FSolverReal HessianDet = ParticleHessian.Determinant();
		auto IsIllConditioned = [](FSolverReal HessianDet)
		{
			return FMath::Abs(HessianDet) < TMathUtilConstants<FSolverReal>::Epsilon ||
				FMath::Abs(HessianDet) > TMathUtilConstants<FSolverReal>::MaxReal;
		};
		if (IsIllConditioned(HessianDet))
		{
			// scale the hessian so that the determinant (~HessianScale^3) falls into the normal range.
			HessianScale = (FSolverReal)0.;
			for (int32 RowIdx = 0; RowIdx < 3; ++RowIdx)
			{
				for (int32 ColIdx = 0; ColIdx < 3; ++ColIdx)
				{
					HessianScale = FMath::Max(FMath::Abs(ParticleHessian.GetAt(RowIdx, ColIdx)), HessianScale);
				}
			}
			if (IsIllConditioned(HessianScale))
			{
				return false;
			}
			else
			{
				ParticleHessian *= 1 / HessianScale;
				HessianDet = ParticleHessian.Determinant();
				if (IsIllConditioned(HessianDet))
				{
					return false;
				}
			}
		}

		Chaos::PMatrix<FSolverReal, 3, 3> HessianInv = ParticleHessian.SymmetricCofactorMatrix();
		HessianInv *= FSolverReal(1) / HessianDet;
		Chaos::TVector<FSolverReal, 3> Dx = HessianInv.GetTransposed() * (-ParticleResidual) / HessianScale; // add back HessianScale

		if (Dx.Size() < MaxDxSize)
		{
			Particles.P(ParticleIdx) += Dx;
		}
		else
		{
			return false;
		}
		return true;
	}

	void FGaussSeidelMainConstraints::InitializeLambdas()
	{
		ComputeInitialResidualAndHessian = [this](const FSolverParticlesRange& Particles, const int32 ParticleIdx, const FSolverReal Dt, TVec3<FSolverReal>& ParticleResidual, Chaos::PMatrix<FSolverReal, 3, 3>& ParticleHessian)
		{
			if (!this->bDoQuasistatics) {
				for (int32 Alpha = 0; Alpha < 3; Alpha++) {
					ParticleResidual[Alpha] = Particles.M(ParticleIdx) * (Particles.P(ParticleIdx)[Alpha] - this->xtilde[ParticleIdx][Alpha]);
				}
				for (int32 Alpha = 0; Alpha < 3; Alpha++) {
					ParticleHessian.SetAt(Alpha, Alpha, Particles.M(ParticleIdx));
				}
			}
			else
			{
				for (int32 Alpha = 0; Alpha < 3; Alpha++) {
					ParticleResidual[Alpha] = -Dt * Dt * ExternalAcceleration[Alpha] * Particles.M(ParticleIdx);
				}
			}
		};

		AccelerationTechniquePerParticle = [this](FSolverParticlesRange& Particles, int32 ParticleIndex)
		{
			if (Particles.InvM(ParticleIndex) != FSolverReal(0) && CurrentIt > SORStart)
			{
				Particles.P(ParticleIndex) = OmegaSOR * (Particles.P(ParticleIndex) - this->X_k_1[ParticleIndex]) + this->X_k_1[ParticleIndex];
			}
			if (Particles.InvM(ParticleIndex) != FSolverReal(0))
			{
				this->X_k_1[ParticleIndex] = this->X_k[ParticleIndex];
				this->X_k[ParticleIndex] = Particles.P(ParticleIndex);
			}
		};
	}

}