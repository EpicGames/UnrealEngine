// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/Deformable/GaussSeidelSphereRepulsionConstraints.h"

namespace Chaos::Softs
{

FGaussSeidelParticleSphereRepulsionConstraints::FGaussSeidelParticleSphereRepulsionConstraints(FSolverReal InRadius, FSolverReal InStiffness, const FSolverParticles& InParticles, const FDeformableXPBDWeightedSpringConstraintParams& InParams)
	: Radius(InRadius), Stiffness(InStiffness), DebugDrawParams(InParams)
{
	const TArrayCollectionArray<FPAndInvM>& PAndInvM = InParticles.GetPAndInvM();
	ReferencePositions.SetNum(PAndInvM.Num());
	for (int32 Index = 0; Index < PAndInvM.Num(); ++Index)
	{
		ReferencePositions[Index] = PAndInvM[Index].P;
	}
}

void FGaussSeidelParticleSphereRepulsionConstraints::AddSphereRepulsionResidualAndHessian(const FSolverParticlesRange& InParticles, const int32 ConstraintIndex, const int32 LocalIndex, const FSolverReal Dt, TVec3<FSolverReal>& ParticleResidual, Chaos::PMatrix<FSolverReal, 3, 3>& ParticleHessian)
{
	const Chaos::TVec3<FSolverReal> X0 = InParticles.P(Constraints[ConstraintIndex][0]);
	const Chaos::TVec3<FSolverReal> X1 = InParticles.P(Constraints[ConstraintIndex][1]);
	const Chaos::TVec3<FSolverReal> Normal = (LocalIndex ? (X1 - X0).GetSafeNormal() : -(X1 - X0).GetSafeNormal()); // dd/dx
	const FSolverReal Dist = (X1 - X0).Size();
	const FSolverReal Penetration = 2 * Radius - Dist; // 2r-d

	if (Penetration > FSolverReal(0))
	{
		const FSolverReal DistInv = FSolverReal(1) / (Dist + FSolverReal(1e-12));
		const Chaos::PMatrix<FSolverReal, 3, 3> Outer = Chaos::PMatrix<FSolverReal, 3, 3>::OuterProduct(Normal, Normal);
		const Chaos::PMatrix<FSolverReal, 3, 3> A = (Chaos::PMatrix<FSolverReal, 3, 3>::Identity - Outer) * DistInv; // (-dd/dx*dd/dx^T+I)/d
		ParticleHessian += Dt * Dt * ConstraintStiffness[ConstraintIndex] * (Outer - Penetration * A);
		ParticleResidual += -Dt * Dt * Penetration * ConstraintStiffness[ConstraintIndex] * Normal;
	}
}

void FGaussSeidelParticleSphereRepulsionConstraints::VisualizeAllBindings(const FSolverParticles& InParticles, const FSolverReal Dt) const
{
#if CHAOS_DEBUG_DRAW
	auto DoubleVert = [](const Chaos::TVec3<FSolverReal>& V) { return FVector3d(V.X, V.Y, V.Z); };
	for (int32 I = 0; I < Constraints.Num(); I++)
	{
		const FVector3d SourcePos = DoubleVert(InParticles.P(Constraints[I][0]));
		const FVector3d TargetPos = DoubleVert(InParticles.P(Constraints[I][1]));

		const float ParticleThickness = DebugDrawParams.DebugParticleWidth;
		const float LineThickness = DebugDrawParams.DebugLineWidth;

		Chaos::FDebugDrawQueue::GetInstance().DrawDebugPoint(SourcePos, FColor::Red, false, Dt, 0, ParticleThickness);
		Chaos::FDebugDrawQueue::GetInstance().DrawDebugPoint(TargetPos, FColor::Red, false, Dt, 0, ParticleThickness);
		Chaos::FDebugDrawQueue::GetInstance().DrawDebugLine(SourcePos, TargetPos, FColor::Green, false, Dt, 0, LineThickness);
	}
#endif
}

void FGaussSeidelParticleSphereRepulsionConstraints::Init(const FSolverParticles& InParticles, const FSolverReal Dt) const
{
	if (DebugDrawParams.bVisualizeBindings)
	{
		VisualizeAllBindings(InParticles, Dt);
	}
}

void FGaussSeidelParticleSphereRepulsionConstraints::UpdateSphereRepulsionConstraints(const FSolverParticles& Particles, const TArray<int32>& SurfaceVertices, const TArray<int32>& ComponentIndex)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(STAT_GaussSeidelSphereRepulsionConstraints);
	Constraints.Reset();
	ConstraintStiffness.Reset();
	if (SurfaceVertices.Num() == 0)
	{
		return;
	}

	// Build Spatial
	TArray<FSphereSpatialEntry> Entries;
	const FSolverReal Diameter = 2.f * Radius;
	const TConstArrayView<FSolverVec3> Points = Particles.XArray();

	Entries.Reset(SurfaceVertices.Num());
	for (int32 Index = 0; Index < SurfaceVertices.Num(); ++Index)
	{
		Entries.Add({ &Points, SurfaceVertices[Index] });
	}

	TSpatialHashGridPoints<int32, FSolverReal> SpatialHash(Diameter);
	SpatialHash.InitializePoints(Entries);

	const FSolverReal DiamSq = FMath::Square(Diameter);
	constexpr int32 CellRadius = 1;
	constexpr int32 MaxNumExpectedConnectionsPerParticle = 3;
	const int32 MaxNumExpectedConnections = MaxNumExpectedConnectionsPerParticle * Entries.Num();

	Constraints = SpatialHash.FindAllSelfProximities(CellRadius, MaxNumExpectedConnections,
		[this, &Particles, DiamSq, &ComponentIndex](const int32 Idx1, const int32 Idx2)
		{
			if (ComponentIndex[Idx1] == ComponentIndex[Idx2])
			{
				return false;
			}
			const FSolverReal CombinedMass = Particles.InvM(Idx1) + Particles.InvM(Idx2);
			if (CombinedMass < (FSolverReal)1e-7)
			{
				return false;
			}
			if (FSolverVec3::DistSquared(this->ReferencePositions[Idx1], this->ReferencePositions[Idx2]) < DiamSq)
			{
				return false;
			}
			return true;
		}
	);

	for (const TVec2<int32>& CollisionPair : Constraints)
	{
		FSolverReal TotalMass = FSolverReal(0);
		if (Particles.InvM(CollisionPair[0]) > (FSolverReal)1e-7)
		{
			TotalMass += Particles.M(CollisionPair[0]);
		}
		if (Particles.InvM(CollisionPair[1]) > (FSolverReal)1e-7)
		{
			TotalMass += Particles.M(CollisionPair[1]);
		}
		ConstraintStiffness.Add(Stiffness * TotalMass / FSolverReal(2));
	}
}

void FGaussSeidelParticleSphereRepulsionConstraints::ReturnSphereRepulsionConstraints(TArray<TArray<int32>>& ExtraConstraints, TArray<TArray<int32>>& ExtraIncidentElements, TArray<TArray<int32>>& ExtraIncidentElementsLocal)
{
	ExtraConstraints.Init(TArray<int32>(), Constraints.Num());
	for (int32 Index = 0; Index < Constraints.Num(); Index++)
	{
		ExtraConstraints[Index].SetNum(2);
		ExtraConstraints[Index][0] = Constraints[Index][0];
		ExtraConstraints[Index][1] = Constraints[Index][1];
	}
	ExtraIncidentElements = Chaos::Utilities::ComputeIncidentElements(ExtraConstraints, &ExtraIncidentElementsLocal);
}

}// End namespace Chaos::Softs
