// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/Deformable/GaussSeidelUnilateralTetConstraints.h"

namespace Chaos::Softs
{

FGaussSeidelUnilateralVolumeConstraints::FGaussSeidelUnilateralVolumeConstraints(
	const FSolverParticles& Particles,
	TArray<TVector<int32, 4>>&& InConstraints,
	TArray<FSolverReal> InStiffnessArray)
	: Constraints(InConstraints)
	, StiffnessArray(InStiffnessArray)
{
	IncidentElements = Chaos::Utilities::ComputeIncidentElements(Constraints, &IncidentElementsLocal);
	Volumes.SetNumUninitialized(Constraints.Num());

	for (int32 Idx = 0; Idx < Constraints.Num(); ++Idx)
	{
		const TVec4<int32>& Constraint = Constraints[Idx];
		Volumes[Idx] = ComputeVolume(
			Particles.X(Constraint[0]),
			Particles.X(Constraint[1]),
			Particles.X(Constraint[2]),
			Particles.X(Constraint[3]));
		StiffnessArray[Idx] /= Volumes[Idx];
	}
}

FSolverReal FGaussSeidelUnilateralVolumeConstraints::ComputeVolume(const TVec3<FSolverReal>& P1, const TVec3<FSolverReal>& P2, const TVec3<FSolverReal>& P3, const TVec3<FSolverReal>& P4)
{
	const TVec3<FSolverReal> P2P1 = P2 - P1;
	const TVec3<FSolverReal> P4P1 = P4 - P1;
	const TVec3<FSolverReal> P3P1 = P3 - P1;
	return TVec3<FSolverReal>::DotProduct(TVec3<FSolverReal>::CrossProduct(P2P1, P3P1), P4P1) / (FSolverReal)6.;
}

void FGaussSeidelUnilateralVolumeConstraints::AddEnergy(const FSolverParticles& InParticles, const int32 ConstraintIndex, const FSolverReal Dt, FSolverReal& Energy) const
{
	const TVec4<int32>& Constraint = Constraints[ConstraintIndex];
	const FSolverReal CurrentVol = ComputeVolume(
		InParticles.P(Constraint[0]),
		InParticles.P(Constraint[1]),
		InParticles.P(Constraint[2]),
		InParticles.P(Constraint[3]));
	Energy += Dt * Dt * StiffnessArray[ConstraintIndex] / (FSolverReal)2. * (CurrentVol - Volumes[ConstraintIndex]) * (CurrentVol - Volumes[ConstraintIndex]);
}

void FGaussSeidelUnilateralVolumeConstraints::AddResidualAndHessian(const FSolverParticles& InParticles, const int32 ConstraintIndex, const int32 LocalIndex, const FSolverReal Dt, TVec3<FSolverReal>& ParticleResidual, Chaos::PMatrix<FSolverReal, 3, 3>& ParticleHessian) const
{
	const TVec4<int32>& Constraint = Constraints[ConstraintIndex];
	const int32 Index1 = Constraint[0];
	const int32 Index2 = Constraint[1];
	const int32 Index3 = Constraint[2];
	const int32 Index4 = Constraint[3];

	const TVec3<FSolverReal>& P1 = InParticles.P(Index1);
	const TVec3<FSolverReal>& P2 = InParticles.P(Index2);
	const TVec3<FSolverReal>& P3 = InParticles.P(Index3);
	const TVec3<FSolverReal>& P4 = InParticles.P(Index4);

	TVec4<TVec3<FSolverReal>> Grads;
	const TVec3<FSolverReal> P2P1 = P2 - P1;
	const TVec3<FSolverReal> P4P1 = P4 - P1;
	const TVec3<FSolverReal> P3P1 = P3 - P1;
	Grads[1] = TVec3<FSolverReal>::CrossProduct(P3P1, P4P1) / (FSolverReal)6.;
	Grads[2] = TVec3<FSolverReal>::CrossProduct(P4P1, P2P1) / (FSolverReal)6.;
	Grads[3] = TVec3<FSolverReal>::CrossProduct(P2P1, P3P1) / (FSolverReal)6.;
	Grads[0] = -(Grads[1] + Grads[2] + Grads[3]);

	const FSolverReal Volume = ComputeVolume(P1, P2, P3, P4);
	const FSolverReal C_Hessian = StiffnessArray[ConstraintIndex] * Dt * Dt;
	const FSolverReal C_Residual = (Volume - Volumes[ConstraintIndex]) * C_Hessian;
	ParticleResidual += C_Residual * Grads[LocalIndex];
	ParticleHessian += C_Hessian * Chaos::PMatrix<FSolverReal, 3, 3>::OuterProduct(Grads[LocalIndex], Grads[LocalIndex]);
}

TArray<TArray<int32>> FGaussSeidelUnilateralVolumeConstraints::GetStaticConstraintArrays(const TArray<TArray<int32>>*& OutIncidentElements, const TArray<TArray<int32>>*& OutIncidentElementsLocal) const
{
	OutIncidentElements = &IncidentElements;
	OutIncidentElementsLocal = &IncidentElementsLocal;
	TArray<TArray<int32>> NestedConstraints;
	NestedConstraints.SetNum(Constraints.Num());
	for (int32 Idx = 0; Idx < NestedConstraints.Num(); ++Idx)
	{
		NestedConstraints[Idx] = {Constraints[Idx][0], Constraints[Idx][1], Constraints[Idx][2], Constraints[Idx][3]};
	}
	return NestedConstraints;
}

TArray<TArray<int32>> FGaussSeidelUnilateralVolumeConstraints::GetStaticConstraintArrays(TArray<TArray<int32>>& InIncidentElements, TArray<TArray<int32>>& InIncidentElementsLocal) const
{
	const TArray<TArray<int32>>* OutIncidentElements = nullptr;
	const TArray<TArray<int32>>* OutIncidentElementsLocal = nullptr;
	TArray<TArray<int32>> Result = GetStaticConstraintArrays(OutIncidentElements, OutIncidentElementsLocal);
	InIncidentElements = *OutIncidentElements;
	InIncidentElementsLocal = *OutIncidentElementsLocal;
	return Result;
}

}// End namespace Chaos::Softs
