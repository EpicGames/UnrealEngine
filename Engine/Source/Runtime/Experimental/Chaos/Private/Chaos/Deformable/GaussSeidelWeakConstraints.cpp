// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/Deformable/GaussSeidelWeakConstraints.h"
#include "Chaos/HierarchicalSpatialHash.h"

namespace Chaos::Softs
{
	using Chaos::TVec3;

	// -------------------------------------------------------------------
	// FGaussSeidelSpringConstraintData
	// -------------------------------------------------------------------

	FGaussSeidelSpringConstraintData::FGaussSeidelSpringConstraintData()
	{
		AddArray(&MIndices);
		AddArray(&MSecondIndices);
		AddArray(&MWeights);
		AddArray(&MSecondWeights);
		AddArray(&MStiffness);
		AddArray(&MIsAnisotropic);
		AddArray(&MNormals);
		AddArray(&MIsZeroRestLength);
		AddArray(&MRestLength);
	}

	FGaussSeidelSpringConstraintData::FGaussSeidelSpringConstraintData(const FGaussSeidelSpringConstraintData& Other)
		: FGaussSeidelSpringConstraintData()
	{
		AddConstraints(Other.Size());
		for (int32 ConstraintIdx = 0; ConstraintIdx < Other.Size(); ++ConstraintIdx)
		{
			SetSingleConstraint(Other.GetSingleConstraintData(ConstraintIdx), ConstraintIdx);
		}
	}

	FGaussSeidelSpringConstraintData::FGaussSeidelSpringConstraintData(FGaussSeidelSpringConstraintData&& Other)
		: TArrayCollection(), MIndices(MoveTemp(Other.MIndices))
		, MSecondIndices(MoveTemp(Other.MSecondIndices))
		, MWeights(MoveTemp(Other.MWeights))
		, MSecondWeights(MoveTemp(Other.MSecondWeights))
		, MStiffness(MoveTemp(Other.MStiffness))
		, MIsAnisotropic(MoveTemp(Other.MIsAnisotropic))
		, MNormals(MoveTemp(Other.MNormals))
		, MIsZeroRestLength(MoveTemp(Other.MIsZeroRestLength))
		, MRestLength(MoveTemp(Other.MRestLength))
	{
		AddArray(&MIndices);
		AddArray(&MSecondIndices);
		AddArray(&MWeights);
		AddArray(&MSecondWeights);
		AddArray(&MStiffness);
		AddArray(&MIsAnisotropic);
		AddArray(&MNormals);
		AddArray(&MIsZeroRestLength);
		AddArray(&MRestLength);
		ResizeHelper(Other.MSize);
		Other.MSize = 0;
	}

	FGaussSeidelSpringConstraintData& FGaussSeidelSpringConstraintData::operator=(FGaussSeidelSpringConstraintData&& Other)
	{
		MIndices = MoveTemp(Other.MIndices);
		MSecondIndices = MoveTemp(Other.MSecondIndices);
		MWeights = MoveTemp(Other.MWeights);
		MSecondWeights = MoveTemp(Other.MSecondWeights);
		MStiffness = MoveTemp(Other.MStiffness);
		MIsAnisotropic = MoveTemp(Other.MIsAnisotropic);
		MNormals = MoveTemp(Other.MNormals);
		MIsZeroRestLength = MoveTemp(Other.MIsZeroRestLength);
		MRestLength = MoveTemp(Other.MRestLength);
		ResizeHelper(Other.Size());
		Other.MSize = 0;
		return *this;
	}

	void FGaussSeidelSpringConstraintData::SetSingleConstraint(const FGaussSeidelSpringConstraintSingleData& SingleData, const int32 ConstraintIndex)
	{
		MIndices[ConstraintIndex] = SingleData.SingleIndices;
		MSecondIndices[ConstraintIndex] = SingleData.SingleSecondIndices;
		MStiffness[ConstraintIndex] = SingleData.SingleStiffness;
		MWeights[ConstraintIndex] = SingleData.SingleWeights;
		MSecondWeights[ConstraintIndex] = SingleData.SingleSecondWeights;
		MNormals[ConstraintIndex] = SingleData.SingleNormal;
		MIsAnisotropic[ConstraintIndex] = SingleData.bIsAnisotropic;
		MIsZeroRestLength[ConstraintIndex] = SingleData.bIsZeroRestLength;
		MRestLength[ConstraintIndex] = SingleData.RestLength;
	}

	void FGaussSeidelSpringConstraintData::AddSingleConstraint(const FGaussSeidelSpringConstraintSingleData& SingleData)
	{
		AddConstraints(1);
		SetSingleConstraint(SingleData, MSize - 1);
	}

	const FGaussSeidelSpringConstraintSingleData FGaussSeidelSpringConstraintData::GetSingleConstraintData(const int32 ConstraintIndex) const
	{
		FGaussSeidelSpringConstraintSingleData SingleConstraintData;
		if (ConstraintIndex > INDEX_NONE && static_cast<uint32>(ConstraintIndex) < MSize)
		{
			SingleConstraintData.SingleIndices = MIndices[ConstraintIndex];
			SingleConstraintData.SingleSecondIndices = MSecondIndices[ConstraintIndex];
			SingleConstraintData.SingleStiffness = MStiffness[ConstraintIndex];
			SingleConstraintData.SingleWeights = MWeights[ConstraintIndex];
			SingleConstraintData.SingleSecondWeights = MSecondWeights[ConstraintIndex];
			SingleConstraintData.bIsAnisotropic = MIsAnisotropic[ConstraintIndex];
			SingleConstraintData.SingleNormal = MNormals[ConstraintIndex];
			SingleConstraintData.bIsZeroRestLength = MIsZeroRestLength[ConstraintIndex];
			SingleConstraintData.RestLength = MRestLength[ConstraintIndex];
		}
		return SingleConstraintData;
	}

	// -------------------------------------------------------------------
	// FGaussSeidelSpringConstraints
	// -------------------------------------------------------------------

	FGaussSeidelSpringConstraints::FGaussSeidelSpringConstraints(
		const TArray<TArray<int32>>& InIndices,
		const TArray<TArray<FSolverReal>>& InWeights,
		const TArray<FSolverReal>& InStiffness,
		const TArray<TArray<int32>>& InSecondIndices,
		const TArray<TArray<FSolverReal>>& InSecondWeights,
		const TArray<bool>& InIsAnisotropic,
		const TArray<bool>& InZeroRestLength,
		const FDeformableXPBDWeightedSpringConstraintParams& InParams
	) : DebugDrawParams(InParams)
	{
		ensureMsgf(InIndices.Num() == InSecondIndices.Num(), TEXT("Input Double Bindings have wrong size"));

		ConstraintsData.AddConstraints(InIndices.Num());

		for (int32 SpringIdx = 0; SpringIdx < InIndices.Num(); ++SpringIdx)
		{
			FGaussSeidelSpringConstraintSingleData SingleConstraintData;
			SingleConstraintData.SingleIndices = InIndices[SpringIdx];
			SingleConstraintData.SingleSecondIndices = InSecondIndices[SpringIdx];
			SingleConstraintData.SingleWeights = InWeights[SpringIdx];
			SingleConstraintData.SingleSecondWeights = InSecondWeights[SpringIdx];
			SingleConstraintData.SingleStiffness = InStiffness[SpringIdx];
			SingleConstraintData.bIsAnisotropic = InIsAnisotropic[SpringIdx];
			SingleConstraintData.bIsZeroRestLength = InZeroRestLength[SpringIdx];
			ConstraintsData.SetSingleConstraint(SingleConstraintData, SpringIdx);
		}
	}

	FGaussSeidelSpringConstraints::FGaussSeidelSpringConstraints(
		const TArray<TArray<int32>>& InIndices,
		const TArray<TArray<FSolverReal>>& InWeights,
		const TArray<FSolverReal>& InStiffness,
		const TArray<TArray<int32>>& InSecondIndices,
		const TArray<TArray<FSolverReal>>& InSecondWeights,
		const FDeformableXPBDWeightedSpringConstraintParams& InParams,
		bool bIsAnisotropic,
		bool bIsZeroRestLength
	) : DebugDrawParams(InParams)
	{
		ensureMsgf(InIndices.Num() == InSecondIndices.Num(), TEXT("Input Double Bindings have wrong size"));

		ConstraintsData.AddConstraints(InIndices.Num());

		for (int32 SpringIdx = 0; SpringIdx < InIndices.Num(); ++SpringIdx)
		{
			FGaussSeidelSpringConstraintSingleData SingleConstraintData;
			SingleConstraintData.SingleIndices = InIndices[SpringIdx];
			SingleConstraintData.SingleSecondIndices = InSecondIndices[SpringIdx];
			SingleConstraintData.SingleWeights = InWeights[SpringIdx];
			SingleConstraintData.SingleSecondWeights = InSecondWeights[SpringIdx];
			SingleConstraintData.SingleStiffness = InStiffness[SpringIdx];
			SingleConstraintData.bIsAnisotropic = bIsAnisotropic;
			SingleConstraintData.bIsZeroRestLength = bIsZeroRestLength;
			ConstraintsData.SetSingleConstraint(SingleConstraintData, SpringIdx);
		}
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FGaussSeidelSpringConstraints::FGaussSeidelSpringConstraints(
		const TArray<TArray<int32>>& InIndices,
		const TArray<TArray<FSolverReal>>& InWeights,
		const TArray<FSolverReal>& InStiffness,
		const TArray<TArray<int32>>& InSecondIndices,
		const TArray<TArray<FSolverReal>>& InSecondWeights,
		const FDeformableXPBDWeightedSpringConstraintParams& InParams
	) : DebugDrawParams(InParams)
	{
		ensureMsgf(InIndices.Num() == InSecondIndices.Num(), TEXT("Input Double Bindings have wrong size"));

		ConstraintsData.AddConstraints(InIndices.Num());

		for (int32 SpringIdx = 0; SpringIdx < InIndices.Num(); SpringIdx++)
		{
			FGaussSeidelSpringConstraintSingleData SingleConstraintData;
			SingleConstraintData.SingleIndices = InIndices[SpringIdx];
			SingleConstraintData.SingleSecondIndices = InSecondIndices[SpringIdx];
			SingleConstraintData.SingleWeights = InWeights[SpringIdx];
			SingleConstraintData.SingleSecondWeights = InSecondWeights[SpringIdx];
			SingleConstraintData.SingleStiffness = InStiffness[SpringIdx];
			ConstraintsData.SetSingleConstraint(SingleConstraintData, SpringIdx);
		}

		for (int32 SpringIdx = 0; SpringIdx < ConstraintsData.Size(); SpringIdx++)
		{
			const TArray<int32>& SingleIndices = ConstraintsData.GetIndices(SpringIdx);
			const TArray<int32>& SingleSecondIndices = ConstraintsData.GetSecondIndices(SpringIdx);
			for (int32 SecondIdx = 0; SecondIdx < SingleSecondIndices.Num(); SecondIdx++)
			{
				ensureMsgf(!SingleIndices.Contains(SingleSecondIndices[SecondIdx]), TEXT("Indices and Second Indices overlaps. Currently not supported"));
			}
		}
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	void FGaussSeidelSpringConstraints::ComputeInitialWCData(const FSolverParticlesRange& InParticles)
	{
		ComputeIncidentElements();
		//Update rest state normal and nodal weights
		UpdateTriangleNormalAndNodalWeight(InParticles, /*bUseParticleX = */true);
		SetNoCollisionData();
		SetConstraintsRestLength(InParticles, InParticles);
	}

	void FGaussSeidelSpringConstraints::ComputeInitialWCData(const FSolverParticles& InParticles)
	{
		ComputeInitialWCData(FSolverParticlesRange(InParticles));
	}

	void FGaussSeidelSpringConstraints::AddWCHessian(const int32 ParticleIdx, const FSolverReal Dt, Chaos::PMatrix<FSolverReal, 3, 3>& ParticleHessian) const
	{
		if (NodalWeights[ParticleIdx].Num() > 0)
		{
			for (int32 Alpha = 0; Alpha < 3; Alpha++)
			{
				ParticleHessian.SetAt(Alpha, Alpha, ParticleHessian.GetAt(Alpha, Alpha) + Dt * Dt * NodalWeights[ParticleIdx][Alpha]);
			}

			ParticleHessian.SetAt(0, 1, ParticleHessian.GetAt(0, 1) + Dt * Dt * NodalWeights[ParticleIdx][3]);
			ParticleHessian.SetAt(0, 2, ParticleHessian.GetAt(0, 2) + Dt * Dt * NodalWeights[ParticleIdx][4]);
			ParticleHessian.SetAt(1, 2, ParticleHessian.GetAt(1, 2) + Dt * Dt * NodalWeights[ParticleIdx][5]);
			ParticleHessian.SetAt(1, 0, ParticleHessian.GetAt(1, 0) + Dt * Dt * NodalWeights[ParticleIdx][3]);
			ParticleHessian.SetAt(2, 0, ParticleHessian.GetAt(2, 0) + Dt * Dt * NodalWeights[ParticleIdx][4]);
			ParticleHessian.SetAt(2, 1, ParticleHessian.GetAt(2, 1) + Dt * Dt * NodalWeights[ParticleIdx][5]);
		}
	}

	void FGaussSeidelSpringConstraints::AddExtraConstraints(const TArray<TArray<int32>>& InIndices,
								const TArray<TArray<FSolverReal>>& InWeights,
								const TArray<FSolverReal>& InStiffness,
								const TArray<TArray<int32>>& InSecondIndices,
								const TArray<TArray<FSolverReal>>& InSecondWeights,
								const TArray<bool>& InIsAnisotropic,
								const TArray<bool>& InIsZeroRestLength)
	{
		const int32 Offset = ConstraintsData.Size();

		ConstraintsData.AddConstraints(InIndices.Num());

		for (int32 SpringIdx = 0; SpringIdx < InIndices.Num(); SpringIdx++)
		{
			FGaussSeidelSpringConstraintSingleData SingleConstraintData;
			SingleConstraintData.SingleIndices = InIndices[SpringIdx];
			SingleConstraintData.SingleSecondIndices = InSecondIndices[SpringIdx];
			SingleConstraintData.SingleWeights = InWeights[SpringIdx];
			SingleConstraintData.SingleSecondWeights = InSecondWeights[SpringIdx];
			SingleConstraintData.SingleStiffness = InStiffness[SpringIdx];
			SingleConstraintData.bIsAnisotropic = InIsAnisotropic[SpringIdx];
			SingleConstraintData.bIsZeroRestLength = InIsZeroRestLength[SpringIdx];
			ConstraintsData.SetSingleConstraint(SingleConstraintData, SpringIdx + Offset);
		}
	}

	void FGaussSeidelSpringConstraints::Init(const FSolverParticlesRange& InParticles, const FSolverReal Dt)
	{
		UpdateTriangleNormalAndNodalWeight(InParticles, /*bUseParticleX =*/ false);
		if (DebugDrawParams.bVisualizeBindings)
		{
			VisualizeAllBindings(InParticles, Dt);
		}
	}

	void FGaussSeidelSpringConstraints::Init(const FSolverParticles& InParticles, const FSolverReal Dt)
	{
		Init(FSolverParticlesRange(InParticles), Dt);
	}

	void FGaussSeidelSpringConstraints::CollisionDetectionBVH(const FSolverParticles& Particles, const TArray<TVec3<int32>>& SurfaceElements, const TArray<int32>& ComponentIndex, float DetectRadius, float PositionTargetStiffness, bool UseAnisotropicSpring)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(STAT_ChaosGaussSeidelWeakConstraintsCollisionDetection);
		Resize(InitialWCSize);

		TArray<Chaos::TVector<int32, 3>> SurfaceElementsArray;
		for (int32 ElemIdx = 0; ElemIdx < SurfaceElements.Num(); ElemIdx++)
		{
			Chaos::TVector<int32, 3> CurrentSurfaceElements(0);
			for (int32 TriLocalIdx = 0; TriLocalIdx < 3; TriLocalIdx++)
			{
				CurrentSurfaceElements[TriLocalIdx] = SurfaceElements[ElemIdx][TriLocalIdx];
			}
			if (CurrentSurfaceElements[0] != INDEX_NONE
				&& CurrentSurfaceElements[1] != INDEX_NONE
				&& CurrentSurfaceElements[2] != INDEX_NONE)
			{
				SurfaceElementsArray.Emplace(CurrentSurfaceElements);
			}
		}
		TArray<TArray<int32>> LocalIndex;
		TArray<TArray<int32>>* LocalIndexPtr = &LocalIndex;
		TArray<TArray<int>> GlobalIndex = Chaos::Utilities::ComputeIncidentElements(SurfaceElementsArray, LocalIndexPtr);
		int32 ActualParticleCount = 0;
		for (int32 ParticleIdx = 0; ParticleIdx < GlobalIndex.Num(); ParticleIdx++)
		{
			if (GlobalIndex[ParticleIdx].Num() > 0)
			{
				ActualParticleCount += 1;
			}
		}
		TArray<Chaos::TVector<float, 3>> SurfaceElementsPositions;
		SurfaceElementsPositions.SetNum(ActualParticleCount);
		TArray<int32> SurfaceElementsMap;
		SurfaceElementsMap.SetNum(ActualParticleCount);
		int32 CurrentParticleIndex = 0;
		for (int32 ParticleIdx = 0; ParticleIdx < GlobalIndex.Num(); ParticleIdx++)
		{
			if (GlobalIndex[ParticleIdx].Num() > 0)
			{
				SurfaceElementsPositions[CurrentParticleIndex] = Particles.P(SurfaceElements[GlobalIndex[ParticleIdx][0]][LocalIndex[ParticleIdx][0]]);
				SurfaceElementsMap[CurrentParticleIndex] = SurfaceElements[GlobalIndex[ParticleIdx][0]][LocalIndex[ParticleIdx][0]];
				CurrentParticleIndex += 1;
			}
		}

		TArray<Chaos::FSphere*> VertexSpherePtrs;
		TArray<Chaos::FSphere> VertexSpheres;

		VertexSpheres.Init(Chaos::FSphere(Chaos::TVec3<Chaos::FReal>(0), DetectRadius), SurfaceElementsPositions.Num());
		VertexSpherePtrs.SetNum(SurfaceElementsPositions.Num());

		for (int32 SphereIdx = 0; SphereIdx < SurfaceElementsPositions.Num(); SphereIdx++)
		{
			Chaos::TVec3<Chaos::FReal> SphereCenter(SurfaceElementsPositions[SphereIdx]);
			Chaos::FSphere VertexSphere(SphereCenter, DetectRadius);
			VertexSpheres[SphereIdx] = Chaos::FSphere(SphereCenter, DetectRadius);
			VertexSpherePtrs[SphereIdx] = &VertexSpheres[SphereIdx];
		}
		Chaos::TBoundingVolumeHierarchy<
			TArray<Chaos::FSphere*>,
			TArray<int32>,
			Chaos::FReal,
			3> VertexBVH(VertexSpherePtrs);

		for (int32 ElemIdx = 0; ElemIdx < SurfaceElements.Num(); ElemIdx++)
		{
			TArray<int32> TriangleIntersections0 = VertexBVH.FindAllIntersections(Particles.P(SurfaceElements[ElemIdx][0]));
			TArray<int32> TriangleIntersections1 = VertexBVH.FindAllIntersections(Particles.P(SurfaceElements[ElemIdx][1]));
			TArray<int32> TriangleIntersections2 = VertexBVH.FindAllIntersections(Particles.P(SurfaceElements[ElemIdx][2]));
			TriangleIntersections0.Sort();
			TriangleIntersections1.Sort();
			TriangleIntersections2.Sort();

			TArray<int32> TriangleIntersections({});
			for (int32 IntersectionIdx = 0; IntersectionIdx < TriangleIntersections0.Num(); IntersectionIdx++)
			{
				if (TriangleIntersections1.Contains(TriangleIntersections0[IntersectionIdx])
					&& TriangleIntersections2.Contains(TriangleIntersections0[IntersectionIdx]))
				{
					TriangleIntersections.Emplace(TriangleIntersections0[IntersectionIdx]);
				}
			}

			const int32 TriangleIndex = ComponentIndex[SurfaceElements[ElemIdx][0]];
			int32 MinIndex = INDEX_NONE;
			float MinDis = DetectRadius;
			Chaos::TVector<float, 3> ClosestBary(0.f);
			Chaos::TVector<float, 3> FaceNormal(0.f);
			for (int32 IntersectionIdx = 0; IntersectionIdx < TriangleIntersections.Num(); IntersectionIdx++)
			{
				if (ComponentIndex[SurfaceElementsMap[TriangleIntersections[IntersectionIdx]]] >= 0 && TriangleIndex >= 0 && ComponentIndex[SurfaceElementsMap[TriangleIntersections[IntersectionIdx]]] != TriangleIndex)
				{
					Chaos::TVector<float, 3> Bary, TriPos0(Particles.P(SurfaceElements[ElemIdx][0])), TriPos1(Particles.P(SurfaceElements[ElemIdx][1])), TriPos2(Particles.P(SurfaceElements[ElemIdx][2])), ParticlePos(Particles.P(SurfaceElementsMap[TriangleIntersections[IntersectionIdx]]));
					Chaos::TVector<Chaos::FRealSingle, 3> ClosestPoint = Chaos::FindClosestPointAndBaryOnTriangle(TriPos0, TriPos1, TriPos2, ParticlePos, Bary);
					const Chaos::FRealSingle CurrentDistance = (Particles.P(SurfaceElementsMap[TriangleIntersections[IntersectionIdx]]) - ClosestPoint).Size();
					if (CurrentDistance < MinDis)
					{
						Chaos::TVector<FSolverReal, 3> Normal = FVector3f::CrossProduct(TriPos2 - TriPos0, TriPos1 - TriPos0); //The normal needs to point outwards of the geometry
						if (FVector3f::DotProduct(ParticlePos - TriPos0, Normal) < 0.f)
						{
							Normal.SafeNormalize(1e-8f);
							MinDis = CurrentDistance;
							MinIndex = SurfaceElementsMap[TriangleIntersections[IntersectionIdx]];
							ClosestBary = Bary;
							FaceNormal = Normal;
						}
					}

				}
			}
			if (MinIndex != INDEX_NONE
				&& MinIndex != SurfaceElements[ElemIdx][0]
				&& MinIndex != SurfaceElements[ElemIdx][1]
				&& MinIndex != SurfaceElements[ElemIdx][2])
			{
				FGaussSeidelSpringConstraintSingleData SingleConstraintData;
				SingleConstraintData.SingleIndices = {SurfaceElements[ElemIdx][0], SurfaceElements[ElemIdx][1], SurfaceElements[ElemIdx][2]};
				SingleConstraintData.SingleSecondIndices = {MinIndex};
				SingleConstraintData.SingleWeights = {ClosestBary[0], ClosestBary[1], ClosestBary[2]};
				SingleConstraintData.SingleSecondWeights = {FSolverReal(1.f)};
				SingleConstraintData.bIsAnisotropic = UseAnisotropicSpring;
				SingleConstraintData.SingleNormal = FaceNormal;
				SingleConstraintData.bIsZeroRestLength = true; //Push-out type collision springs should be zero rest length

				float SpringStiffness = 0.f;
				for (int32 TriLocalIdx = 0; TriLocalIdx < 3; TriLocalIdx++)
				{
					SpringStiffness += ClosestBary[TriLocalIdx] * PositionTargetStiffness * Particles.M(SurfaceElements[ElemIdx][TriLocalIdx]);
				}
				SpringStiffness += PositionTargetStiffness * Particles.M(MinIndex);
				SingleConstraintData.SingleStiffness = (FSolverReal)SpringStiffness;
				ConstraintsData.AddSingleConstraint(SingleConstraintData);
			}
		}
	}

	void FGaussSeidelSpringConstraints::ComputeCollisionWCDataSimplified(TArray<TArray<int32>>& ExtraConstraints, TArray<TArray<int32>>& ExtraWCIncidentElements, TArray<TArray<int32>>& ExtraWCIncidentElementsLocal)
	{
		ensureMsgf(ConstraintsData.Size() >= InitialWCSize, TEXT("The size of Indices is smaller than InitialWCSize"));

		ExtraConstraints.Init(TArray<int32>(), ConstraintsData.Size() - InitialWCSize);
		for (int32 ConstraintIdx = InitialWCSize; ConstraintIdx < static_cast<int32>(ConstraintsData.Size()); ConstraintIdx++)
		{
			const TArray<int32>& LocalIndices = ConstraintsData.GetIndices(ConstraintIdx);
			const TArray<int32>& LocalSecondIndices = ConstraintsData.GetSecondIndices(ConstraintIdx);

			ExtraConstraints[ConstraintIdx - InitialWCSize].SetNum(LocalIndices.Num() + LocalSecondIndices.Num());
			for (int32 LocalIdx = 0; LocalIdx < LocalIndices.Num(); LocalIdx++)
			{
				ExtraConstraints[ConstraintIdx - InitialWCSize][LocalIdx] = LocalIndices[LocalIdx];
			}
			for (int32 SecondLocalIdx = 0; SecondLocalIdx < LocalSecondIndices.Num(); SecondLocalIdx++)
			{
				ExtraConstraints[ConstraintIdx - InitialWCSize][SecondLocalIdx + LocalIndices.Num()] = LocalSecondIndices[SecondLocalIdx];
			}
		}

		ExtraWCIncidentElements = Chaos::Utilities::ComputeIncidentElements(ExtraConstraints, &ExtraWCIncidentElementsLocal);

		NodalWeights = NoCollisionNodalWeights;
		for (int32 IncidentIdx = 0; IncidentIdx < ExtraWCIncidentElements.Num(); IncidentIdx++)
		{
			if (ExtraWCIncidentElements[IncidentIdx].Num() > 0)
			{
				const int32 ParticleIdx = ExtraConstraints[ExtraWCIncidentElements[IncidentIdx][0]][ExtraWCIncidentElementsLocal[IncidentIdx][0]];
				if (NodalWeights[ParticleIdx].Num() == 0)
				{
					NodalWeights[ParticleIdx].Init(FSolverReal(0), 6);
				}
				for (int32 LocalIdx = 0; LocalIdx < ExtraWCIncidentElements[IncidentIdx].Num(); LocalIdx++)
				{
					const int32 LocalIndex = ExtraWCIncidentElementsLocal[IncidentIdx][LocalIdx];
					const int32 ConstraintIndex = ExtraWCIncidentElements[IncidentIdx][LocalIdx] + InitialWCSize;

					const FGaussSeidelSpringConstraintSingleData& SingleData = ConstraintsData.GetSingleConstraintData(ConstraintIndex);

					FSolverReal Weight = FSolverReal(0);
					if (LocalIndex >= SingleData.SingleIndices.Num())
					{
						Weight = SingleData.SingleWeights[LocalIndex - SingleData.SingleIndices.Num()];
					}
					else
					{
						Weight = SingleData.SingleWeights[LocalIndex];
					}
					const FSolverReal WeightedStiffness = Weight * Weight * SingleData.SingleStiffness;
					if (SingleData.bIsAnisotropic)
					{
						const TVec3<FSolverReal>& Normal = SingleData.SingleNormal;
						for (int32 Alpha = 0; Alpha < 3; Alpha++)
						{
							NodalWeights[ParticleIdx][Alpha] += Normal[Alpha] * Normal[Alpha] * WeightedStiffness;
						}

						NodalWeights[ParticleIdx][3] += Normal[0] * Normal[1] * WeightedStiffness;
						NodalWeights[ParticleIdx][4] += Normal[0] * Normal[2] * WeightedStiffness;
						NodalWeights[ParticleIdx][5] += Normal[1] * Normal[2] * WeightedStiffness;
					}
					else
					{
						for (int32 Alpha = 0; Alpha < 3; Alpha++)
						{
							NodalWeights[ParticleIdx][Alpha] += WeightedStiffness;
						}
					}
				}
			}
		}
	}

	const TArray<TArray<int32>>& FGaussSeidelSpringConstraints::GetStaticConstraintArrays(const TArray<TArray<int32>>*& OutIncidentElements, const TArray<TArray<int32>>*& OutIncidentElementsLocal) const
	{
		OutIncidentElements = &NoCollisionWCIncidentElements;
		OutIncidentElementsLocal = &NoCollisionWCIncidentElementsLocal;
		return NoCollisionConstraints;
	}

	const TArray<TArray<int32>>& FGaussSeidelSpringConstraints::GetStaticConstraintArrays(TArray<TArray<int32>>& IncidentElements, TArray<TArray<int32>>& IncidentElementsLocal) const
	{
		IncidentElements = NoCollisionWCIncidentElements;
		IncidentElementsLocal = NoCollisionWCIncidentElementsLocal;
		return NoCollisionConstraints;
	}

	TArray<TArray<int32>> FGaussSeidelSpringConstraints::GetDynamicConstraintArrays(TArray<TArray<int32>>& IncidentElements, TArray<TArray<int32>>& IncidentElementsLocal) const
	{
		TArray<TArray<int32>> ExtraConstraints;
		ExtraConstraints.Init(TArray<int32>(), ConstraintsData.Size());
		for (int32 ConstraintIdx = InitialWCSize; ConstraintIdx < ConstraintsData.Size(); ConstraintIdx++)
		{
			const TArray<int32>& LocalIndices = ConstraintsData.GetIndices(ConstraintIdx);
			const TArray<int32>& LocalSecondIndices = ConstraintsData.GetSecondIndices(ConstraintIdx);
			ExtraConstraints[ConstraintIdx - InitialWCSize].SetNum(LocalIndices.Num() + LocalSecondIndices.Num());
			for (int32 LocalIdx = 0; LocalIdx < LocalIndices.Num(); LocalIdx++)
			{
				ExtraConstraints[ConstraintIdx - InitialWCSize][LocalIdx] = LocalIndices[LocalIdx];
			}
			for (int32 SecondLocalIdx = 0; SecondLocalIdx < LocalSecondIndices.Num(); SecondLocalIdx++)
			{
				ExtraConstraints[ConstraintIdx - InitialWCSize][SecondLocalIdx + LocalIndices.Num()] = LocalSecondIndices[SecondLocalIdx];
			}
		}

		IncidentElements = Chaos::Utilities::ComputeIncidentElements(ExtraConstraints, &IncidentElementsLocal);

		return ExtraConstraints;
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	void FGaussSeidelSpringConstraints::AddZeroRestLengthWCResidualAndHessian(const FSolverParticles& InParticles, const int32 ConstraintIndex, const int32 LocalIndex, const FSolverReal Dt, TVec3<FSolverReal>& ParticleResidual, Chaos::PMatrix<FSolverReal, 3, 3>& ParticleHessian) const
	{
		const FGaussSeidelSpringConstraintSingleData& SingleData = ConstraintsData.GetSingleConstraintData(ConstraintIndex);

		const TVec3<FSolverReal> SpringEdge = ComputeSpringEdge(InParticles, SingleData.SingleIndices, SingleData.SingleSecondIndices,
			SingleData.SingleWeights, SingleData.SingleSecondWeights, /*bUseParticleX =*/false);
		FSolverReal Weight = FSolverReal(0);
		if (LocalIndex >= SingleData.SingleIndices.Num())
		{
			Weight = -SingleData.SingleSecondWeights[LocalIndex - SingleData.SingleIndices.Num()];
		}
		else
		{
			Weight = SingleData.SingleWeights[LocalIndex];
		}
		if (SingleData.bIsAnisotropic)
		{
			const FSolverReal Comp = TVec3<FSolverReal>::DotProduct(SpringEdge, SingleData.SingleNormal);
			const TVec3<FSolverReal> Proj = SingleData.SingleNormal * Comp;
			for (int32 Alpha = 0; Alpha < 3; Alpha++)
			{
				ParticleResidual[Alpha] += Dt * Dt * SingleData.SingleStiffness * Proj[Alpha] * Weight;
			}
		}
		else
		{
			for (int32 Alpha = 0; Alpha < 3; Alpha++)
			{
				ParticleResidual[Alpha] += Dt * Dt * SingleData.SingleStiffness * SpringEdge[Alpha] * Weight;
			}
		}
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	void FGaussSeidelSpringConstraints::AddWCResidual(const FSolverParticlesRange& InParticles, const int32 ConstraintIndex, const int32 LocalIndex, const FSolverReal Dt, TVec3<FSolverReal>& ParticleResidual, Chaos::PMatrix<FSolverReal, 3, 3>& ParticleHessian) const
	{
		const FGaussSeidelSpringConstraintSingleData& SingleData = ConstraintsData.GetSingleConstraintData(ConstraintIndex);

		const TVec3<FSolverReal> SpringEdge = ComputeSpringEdge(InParticles, SingleData.SingleIndices, SingleData.SingleSecondIndices,
			SingleData.SingleWeights, SingleData.SingleSecondWeights, /*bUseParticleX =*/false);
		FSolverReal Weight = FSolverReal(0);
		if (LocalIndex >= SingleData.SingleIndices.Num())
		{
			Weight = -SingleData.SingleSecondWeights[LocalIndex - SingleData.SingleIndices.Num()];
		}
		else
		{
			Weight = SingleData.SingleWeights[LocalIndex];
		}
		TVec3<FSolverReal> Projection;
		if (SingleData.bIsAnisotropic)
		{
			const FSolverReal LengthDiff = TVec3<FSolverReal>::DotProduct(SpringEdge, SingleData.SingleNormal) - SingleData.RestLength;
			Projection = SingleData.SingleNormal * LengthDiff;
		}
		else
		{
			Projection = SpringEdge;
			if (!SingleData.bIsZeroRestLength) // if not zero rest-length, apply repulsion force
			{
				Projection -= SingleData.RestLength * SpringEdge.GetSafeNormal();
			}
		}
		for (int32 Alpha = 0; Alpha < 3; ++Alpha)
		{
			ParticleResidual[Alpha] += Dt * Dt * SingleData.SingleStiffness * Projection[Alpha] * Weight;
		}
	}

	void FGaussSeidelSpringConstraints::AddWCResidual(const FSolverParticles& InParticles, const int32 ConstraintIndex, const int32 LocalIndex, const FSolverReal Dt, TVec3<FSolverReal>& ParticleResidual, Chaos::PMatrix<FSolverReal, 3, 3>& ParticleHessian) const
	{
		AddWCResidual(FSolverParticlesRange(InParticles), ConstraintIndex, LocalIndex, Dt, ParticleResidual, ParticleHessian);
	}

	void FGaussSeidelSpringConstraints::SetConstraintsRestLength(const FSolverParticlesRange& InParticles, const FSolverParticlesRange& InSecondParticles)
	{
		for (int32 ConstraintIdx = 0; ConstraintIdx < ConstraintsData.Size(); ++ConstraintIdx)
		{
			if (ConstraintsData.GetIsZeroRestLength(ConstraintIdx))
			{
				ConstraintsData.SetRestLength(ConstraintIdx, (FSolverReal)0.);
			}
			else
			{
				const TArray<int32>& Indices = ConstraintsData.GetIndices(ConstraintIdx);
				const TArray<int32>& SecondIndices = ConstraintsData.GetSecondIndices(ConstraintIdx);
				const TArray<FSolverReal>& Weights = ConstraintsData.GetWeights(ConstraintIdx);
				const TArray<FSolverReal>& SecondWeights = ConstraintsData.GetSecondWeights(ConstraintIdx);
				const TVec3<FSolverReal> RestSpringEdge = ComputeSpringEdge(InParticles, InSecondParticles, Indices, SecondIndices, Weights, SecondWeights, /*bUseParticleX =*/true);
				if (ConstraintsData.GetIsAnisotropic(ConstraintIdx))
				{
					//if the spring is anisotropic, rest length could be negative depending on the normal direction
					ConstraintsData.SetRestLength(ConstraintIdx, TVec3<FSolverReal>::DotProduct(ConstraintsData.GetNormal(ConstraintIdx), RestSpringEdge));
				}
				else
				{
					ConstraintsData.SetRestLength(ConstraintIdx, RestSpringEdge.Size());
				}
			}
		}
	}

	void FGaussSeidelSpringConstraints::UpdateTriangleNormalAndNodalWeight(const FSolverParticlesRange& InParticles, const FSolverParticlesRange& InSecondParticles, bool bUseParticleX)
	{
		UpdateTriangleNormal(InParticles, InSecondParticles, bUseParticleX);
		UpdateNodalWeight(InParticles, InSecondParticles, bUseParticleX);
	}

	TVec3<FSolverReal> FGaussSeidelSpringConstraints::ComputeSpringEdge(const FSolverParticlesRange& InParticles, const FSolverParticlesRange& InSecondParticles,
		const TArray<int32>& LocalIndices, const TArray<int32>& LocalSecondIndices,
		const TArray<FSolverReal>& Weight, const TArray<FSolverReal>& SecondWeight, bool bUseParticleX) const
	{
		auto GetPosition = [bUseParticleX](const FSolverParticlesRange& InParticles, int32 Idx) -> const TVec3<FSolverReal>&
			{
				return bUseParticleX ? InParticles.X(Idx) : InParticles.P(Idx);
			};
		TVec3<FSolverReal> SpringEdge((FSolverReal)0.);
		if (ensure(LocalIndices.Num() == Weight.Num() && LocalSecondIndices.Num() == SecondWeight.Num()))
		{
			for (int32 Idx = 0; Idx < LocalIndices.Num(); ++Idx)
			{
				SpringEdge += Weight[Idx] * GetPosition(InParticles, LocalIndices[Idx]);
			}
			for (int32 SecondIdx = 0; SecondIdx < LocalSecondIndices.Num(); ++SecondIdx)
			{
				SpringEdge -= SecondWeight[SecondIdx] * GetPosition(InSecondParticles, LocalSecondIndices[SecondIdx]);
			}
		}
		return SpringEdge;
	}

	TVec3<FSolverReal> FGaussSeidelSpringConstraints::ComputeSpringEdge(const FSolverParticles& InParticles, const FSolverParticles& InSecondParticles,
		const TArray<int32>& LocalIndices, const TArray<int32>& LocalSecondIndices,
		const TArray<FSolverReal>& Weight, const TArray<FSolverReal>& SecondWeight, bool bUseParticleX) const
	{
		return ComputeSpringEdge(FSolverParticlesRange(InParticles),
			FSolverParticlesRange(InSecondParticles),
			LocalIndices, LocalSecondIndices, Weight, SecondWeight, bUseParticleX);
	}

	TVec3<FSolverReal> FGaussSeidelSpringConstraints::ComputeSpringEdge(const FSolverParticlesRange& InParticles,
		const TArray<int32>& LocalIndices, const TArray<int32>& LocalSecondIndices,
		const TArray<FSolverReal>& Weight, const TArray<FSolverReal>& SecondWeight, bool bUseParticleX) const
	{
		return ComputeSpringEdge(InParticles, InParticles, LocalIndices, LocalSecondIndices, Weight, SecondWeight, bUseParticleX);
	}

	TVec3<FSolverReal> FGaussSeidelSpringConstraints::ComputeSpringEdge(const FSolverParticles& InParticles, const TArray<int32>& LocalIndices,
		const TArray<int32>& LocalSecondIndices, const TArray<FSolverReal>& Weight, const TArray<FSolverReal>& SecondWeight, bool bUseParticleX) const
	{
		return ComputeSpringEdge(FSolverParticlesRange(InParticles),
			LocalIndices, LocalSecondIndices, Weight, SecondWeight, bUseParticleX);
	}

	void FGaussSeidelSpringConstraints::SetNoCollisionData()
	{
		InitialWCSize = ConstraintsData.Size();
		NoCollisionNodalWeights = NodalWeights;
		NoCollisionWCIncidentElements = WCIncidentElements;
		NoCollisionWCIncidentElementsLocal = WCIncidentElementsLocal;
	}

	void FGaussSeidelSpringConstraints::ComputeIncidentElements()
	{
		TArray<TArray<int32>> ExtraConstraints;
		ExtraConstraints.Init(TArray<int32>(), ConstraintsData.Size());
		for (int32 ConstraintIdx = 0; ConstraintIdx < ExtraConstraints.Num(); ++ConstraintIdx)
		{
			ExtraConstraints[ConstraintIdx].SetNum(ConstraintsData.GetIndices(ConstraintIdx).Num() + ConstraintsData.GetSecondIndices(ConstraintIdx).Num());
			for (int32 LocalIdx = 0; LocalIdx < ConstraintsData.GetIndices(ConstraintIdx).Num(); ++LocalIdx)
			{
				ExtraConstraints[ConstraintIdx][LocalIdx] = ConstraintsData.GetIndices(ConstraintIdx)[LocalIdx];
			}
			for (int32 LocalSecondIdx = 0; LocalSecondIdx < ConstraintsData.GetSecondIndices(ConstraintIdx).Num(); ++LocalSecondIdx)
			{
				ExtraConstraints[ConstraintIdx][LocalSecondIdx + ConstraintsData.GetIndices(ConstraintIdx).Num()] = ConstraintsData.GetSecondIndices(ConstraintIdx)[LocalSecondIdx];
			}
		}
		WCIncidentElements = Chaos::Utilities::ComputeIncidentElements(ExtraConstraints, &WCIncidentElementsLocal);
		NoCollisionConstraints = ExtraConstraints;
	}

	void FGaussSeidelSpringConstraints::UpdateTriangleNormal(const FSolverParticlesRange& InParticles, const FSolverParticlesRange& InSecondParticles, bool bUseParticleX)
	{
		auto GetPosition = [bUseParticleX](const FSolverParticlesRange& InParticles, int32 Idx) -> const TVec3<FSolverReal>&
			{
				return bUseParticleX ? InParticles.X(Idx) : InParticles.P(Idx);
			};
		for (int32 ConstraintIdx = 0; ConstraintIdx < ConstraintsData.Size(); ConstraintIdx++)
		{
			if (ConstraintsData.GetIsAnisotropic(ConstraintIdx))
			{
				const TArray<int32>& IndicesTemp = ConstraintsData.GetIndices(ConstraintIdx);
				const TArray<int32>& SecondIndicesTemp = ConstraintsData.GetSecondIndices(ConstraintIdx);
				Chaos::TVector<FSolverReal, 3> Normal((FSolverReal)0.);
				if (IndicesTemp.Num() == 3)
				{
					Chaos::TVector<float, 3> TriPos0(GetPosition(InParticles, IndicesTemp[0])),
						TriPos1(GetPosition(InParticles, IndicesTemp[1])),
						TriPos2(GetPosition(InParticles, IndicesTemp[2]));
					Normal = FVector3f::CrossProduct(TriPos2 - TriPos0, TriPos1 - TriPos0).GetSafeNormal(); //triangle normal convention (see FTriangleMesh::GetFaceNormals())
				}
				else if (SecondIndicesTemp.Num() == 3)
				{
					Chaos::TVector<float, 3> TriPos0(GetPosition(InSecondParticles, SecondIndicesTemp[0])),
						TriPos1(GetPosition(InSecondParticles, SecondIndicesTemp[1])),
						TriPos2(GetPosition(InSecondParticles, SecondIndicesTemp[2]));
					Normal = FVector3f::CrossProduct(TriPos2 - TriPos0, TriPos1 - TriPos0).GetSafeNormal(); //triangle normal convention (see FTriangleMesh::GetFaceNormals())
				}
				ConstraintsData.SetNormal(ConstraintIdx, Normal);
			}
		}
	}

	void FGaussSeidelSpringConstraints::UpdateNodalWeight(const FSolverParticlesRange& InParticles, const FSolverParticlesRange& InSecondParticles, bool bUseParticleX)
	{
		NodalWeights.Init({}, InParticles.Size());
		for (int32 ParticleIdx = 0; ParticleIdx < WCIncidentElements.Num(); ++ParticleIdx)
		{
			if (WCIncidentElements[ParticleIdx].Num() > 0)
			{
				NodalWeights[ParticleIdx].Init(FSolverReal(0), 6);
				for (int32 LocalIdx = 0; LocalIdx < WCIncidentElements[ParticleIdx].Num(); LocalIdx++)
				{
					const int32 ConstraintIndex = WCIncidentElements[ParticleIdx][LocalIdx];
					const int32 LocalIndex = WCIncidentElementsLocal[ParticleIdx][LocalIdx];

					FSolverReal Weight = FSolverReal(0);
					if (LocalIndex >= ConstraintsData.GetIndices(ConstraintIndex).Num())
					{
						Weight = ConstraintsData.GetSecondWeights(ConstraintIndex)[LocalIndex - ConstraintsData.GetIndices(ConstraintIndex).Num()];
					}
					else
					{
						Weight = ConstraintsData.GetWeights(ConstraintIndex)[LocalIndex];
					}

					const FSolverReal WeightedStiffness = Weight * Weight * ConstraintsData.GetStiffness(ConstraintIndex);
					if (ConstraintsData.GetIsAnisotropic(ConstraintIndex))
					{
						const TVec3<FSolverReal>& Normal = ConstraintsData.GetNormal(ConstraintIndex);
						for (int32 Alpha = 0; Alpha < 3; Alpha++)
						{
							NodalWeights[ParticleIdx][Alpha] += Normal[Alpha] * Normal[Alpha] * WeightedStiffness;
						}

						NodalWeights[ParticleIdx][3] += Normal[0] * Normal[1] * WeightedStiffness;
						NodalWeights[ParticleIdx][4] += Normal[0] * Normal[2] * WeightedStiffness;
						NodalWeights[ParticleIdx][5] += Normal[1] * Normal[2] * WeightedStiffness;
					}
					else
					{
						for (int32 Alpha = 0; Alpha < 3; Alpha++)
						{
							NodalWeights[ParticleIdx][Alpha] += WeightedStiffness;
						}
					}
				}
			}
		}
	}

	void FGaussSeidelSpringConstraints::UpdateTriangleNormalAndNodalWeight(const FSolverParticlesRange& InParticles, bool bUseParticleX)
	{
		UpdateTriangleNormalAndNodalWeight(InParticles, InParticles, bUseParticleX);
	}

	void FGaussSeidelSpringConstraints::Resize(int32 Size)
	{
		ConstraintsData.Resize(Size);
	}

	void FGaussSeidelSpringConstraints::UpdatePointTriangleCollisionWCData(const FSolverParticles& Particles)
	{
		FGaussSeidelSpringConstraintData OriginalConstraintsData = ConstraintsData;

		ConstraintsData.Resize(InitialWCSize);

		for (int32 ConstraintIdx = InitialWCSize; ConstraintIdx < OriginalConstraintsData.Size(); ConstraintIdx++)
		{
			const TArray<int32>& IndicesTemp = OriginalConstraintsData.GetIndices(ConstraintIdx);
			const TArray<int32>& SecondIndicesTemp = OriginalConstraintsData.GetSecondIndices(ConstraintIdx);
			ensureMsgf(OriginalConstraintsData.GetIndices(ConstraintIdx).Num() == 3, TEXT("Collision format is not point-triangle"));
			ensureMsgf(OriginalConstraintsData.GetSecondIndices(ConstraintIdx).Num() == 1, TEXT("Collision format is not point-triangle"));
			Chaos::TVector<float, 3> TriPos0(Particles.P(IndicesTemp[0])), TriPos1(Particles.P(IndicesTemp[1])), TriPos2(Particles.P(IndicesTemp[2])), ParticlePos(Particles.P(SecondIndicesTemp[0]));
			Chaos::TVector<FSolverReal, 3> Normal = FVector3f::CrossProduct(TriPos2 - TriPos0, TriPos1 - TriPos0); //triangle normal convention (see FTriangleMesh::GetFaceNormals())
			if (FVector3f::DotProduct(ParticlePos - TriPos0, Normal) < 0.f) //not resolved, keep the spring
			{
				ConstraintsData.AddSingleConstraint(OriginalConstraintsData.GetSingleConstraintData(ConstraintIdx));
			}
		}
	}

	void FGaussSeidelSpringConstraints::VisualizeAllBindings(const FSolverParticlesRange& InParticles, const FSolverReal Dt) const
	{
#if CHAOS_DEBUG_DRAW 
		auto DoubleVert = [](const Chaos::TVec3<FSolverReal>& V) { return FVector3d(V.X, V.Y, V.Z); };
		for (int32 ConstraintIdx = 0; ConstraintIdx < ConstraintsData.Size(); ConstraintIdx++)
		{
			const FGaussSeidelSpringConstraintSingleData& SingleConstraintData = ConstraintsData.GetSingleConstraintData(ConstraintIdx);
			Chaos::TVec3<FSolverReal> SourcePos((FSolverReal)0.), TargetPos((FSolverReal)0.);
			for (int32 Idx = 0; Idx < SingleConstraintData.SingleIndices.Num(); Idx++)
			{
				SourcePos += SingleConstraintData.SingleWeights[Idx] * InParticles.P(SingleConstraintData.SingleIndices[Idx]);
			}
			for (int32 Idx = 0; Idx < SingleConstraintData.SingleSecondIndices.Num(); Idx++)
			{
				TargetPos += SingleConstraintData.SingleSecondWeights[Idx] * InParticles.P(SingleConstraintData.SingleSecondIndices[Idx]);
			}

			const float ParticleThickness = DebugDrawParams.DebugParticleWidth;
			const float LineThickness = DebugDrawParams.DebugLineWidth;

			if (SingleConstraintData.SingleIndices.Num() == 1)
			{
				Chaos::FDebugDrawQueue::GetInstance().DrawDebugPoint(DoubleVert(SourcePos), FColor::Red, false, Dt, 0, ParticleThickness);
				for (int32 Idx = 0; Idx < SingleConstraintData.SingleSecondIndices.Num(); Idx++)
				{
					Chaos::FDebugDrawQueue::GetInstance().DrawDebugPoint(DoubleVert(InParticles.P(SingleConstraintData.SingleSecondIndices[Idx])), FColor::Green, false, Dt, 0, ParticleThickness);
					Chaos::FDebugDrawQueue::GetInstance().DrawDebugLine(DoubleVert(InParticles.P(SingleConstraintData.SingleSecondIndices[Idx])), DoubleVert(InParticles.P(SingleConstraintData.SingleSecondIndices[(Idx + 1) % SingleConstraintData.SingleSecondIndices.Num()])), FColor::Green, false, Dt, 0, LineThickness);
				}

			}

			if (SingleConstraintData.SingleSecondIndices.Num() == 1)
			{
				Chaos::FDebugDrawQueue::GetInstance().DrawDebugPoint(DoubleVert(TargetPos), FColor::Red, false, Dt, 0, ParticleThickness);
				for (int32 Idx = 0; Idx < SingleConstraintData.SingleIndices.Num(); Idx++)
				{
					Chaos::FDebugDrawQueue::GetInstance().DrawDebugPoint(DoubleVert(InParticles.P(SingleConstraintData.SingleIndices[Idx])), FColor::Green, false, Dt, 0, ParticleThickness);
					Chaos::FDebugDrawQueue::GetInstance().DrawDebugLine(DoubleVert(InParticles.P(SingleConstraintData.SingleIndices[Idx])), DoubleVert(InParticles.P(SingleConstraintData.SingleIndices[(Idx + 1) % SingleConstraintData.SingleIndices.Num()])), FColor::Green, false, Dt, 0, LineThickness);
				}
			}

			Chaos::FDebugDrawQueue::GetInstance().DrawDebugLine(DoubleVert(SourcePos), DoubleVert(TargetPos), FColor::Yellow, false, Dt, 0, LineThickness);
		}
#endif
	}

	// -------------------------------------------------------------------
	// CollisionDetectionSpatialHash / CollisionDetectionSpatialHashInComponent
	// -------------------------------------------------------------------

	template<typename SpatialAccelerator>
	void FGaussSeidelSpringConstraints::CollisionDetectionSpatialHash(const FSolverParticles& Particles, const TArray<int32>& SurfaceVertices, const FTriangleMesh& TriangleMesh, const TArray<int32>& ComponentIndex, const SpatialAccelerator& Spatial, float DetectRadius, float PositionTargetStiffness, bool UseAnisotropicSpring)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(STAT_ChaosGaussSeidelWeakConstraintsCollisionDetectionSpatialHash);
		Resize(InitialWCSize + Particles.Size());
		std::atomic<int32> ConstraintIndex(InitialWCSize);
		const TArray<TVec3<int32>>& Elements = TriangleMesh.GetSurfaceElements();
		const float HalfRadius = DetectRadius / 2;
		PhysicsParallelFor(SurfaceVertices.Num(),
			[this, &Spatial, &Particles, &SurfaceVertices, &ConstraintIndex, &TriangleMesh, &Elements, &HalfRadius, &ComponentIndex, &PositionTargetStiffness, &UseAnisotropicSpring](int32 SurfaceVdx)
			{
				const int32 Index = SurfaceVertices[SurfaceVdx];
				TArray< TTriangleCollisionPoint<FSolverReal>> Result;
				//PointProximityQuery
				if (TriangleMesh.PointClosestTriangleQuery(Spatial, static_cast<const TArrayView<const FSolverVec3>&>(Particles.XArray()), Index, Particles.GetX(Index), HalfRadius, HalfRadius,
					[this, &ComponentIndex, &Elements](const int32 PointIndex, const int32 TriangleIndex)->bool
					{
						//Skip particles that are bound in initial springs
						return ComponentIndex[PointIndex] != ComponentIndex[Elements[TriangleIndex][0]] && (!NoCollisionWCIncidentElements.IsValidIndex(PointIndex) || NoCollisionWCIncidentElements[PointIndex].Num() == 0);
					},
					Result))
				{
					for (const TTriangleCollisionPoint<FSolverReal>& CollisionPoint : Result)
					{
						if (CollisionPoint.Phi < 0)
						{
							const TVector<int32, 3>& Elem = Elements[CollisionPoint.Indices[1]];
							const int32 IndexToWrite = ConstraintIndex.fetch_add(1);

							FGaussSeidelSpringConstraintSingleData SingleConstraintData;
							SingleConstraintData.SingleIndices = { Elem[0], Elem[1] ,Elem[2] };
							SingleConstraintData.SingleSecondIndices =  { Index };
							SingleConstraintData.SingleWeights = { CollisionPoint.Bary[1], CollisionPoint.Bary[2], CollisionPoint.Bary[3] };
							SingleConstraintData.SingleSecondWeights = {FSolverReal(1.f)};
							SingleConstraintData.bIsAnisotropic = UseAnisotropicSpring;
							SingleConstraintData.SingleNormal = CollisionPoint.Normal;
							SingleConstraintData.bIsZeroRestLength = true; //Push-out type collision springs should be zero rest length

							float SpringStiffness = 0.f;
							for (int32 TriLocalIdx = 0; TriLocalIdx < 3; TriLocalIdx++)
							{
								SpringStiffness += SingleConstraintData.SingleWeights[TriLocalIdx] * PositionTargetStiffness * Particles.M(Elem[TriLocalIdx]);
							}
							SpringStiffness += PositionTargetStiffness * Particles.M(Index);
							SingleConstraintData.SingleStiffness = (FSolverReal)SpringStiffness;
							ConstraintsData.SetSingleConstraint(SingleConstraintData, IndexToWrite);
						}
					}
				}
			}
		);

		// Shrink the arrays to the actual number of found constraints.
		const int32 ConstraintNum = ConstraintIndex.load();
		Resize(ConstraintNum);
	}

	template<typename SpatialAccelerator>
	void FGaussSeidelSpringConstraints::CollisionDetectionSpatialHashInComponent(const FSolverParticles& Particles, const TArray<int32>& SurfaceVertices, const FTriangleMesh& TriangleMesh, const TMap<int32, TSet<int32>>& ExcludeMap, const SpatialAccelerator& Spatial, float DetectRadius, float PositionTargetStiffness, bool UseAnisotropicSpring)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(STAT_ChaosGaussSeidelWeakConstraintsCollisionDetectionSpatialHashInComponent);
		Resize(InitialWCSize + Particles.Size());
		std::atomic<int32> ConstraintIndex(InitialWCSize);
		const TArray<TVec3<int32>>& Elements = TriangleMesh.GetSurfaceElements();
		const float HalfRadius = DetectRadius/2;
		PhysicsParallelFor(SurfaceVertices.Num(),
			[this, &Spatial, &Particles, &SurfaceVertices, &ConstraintIndex, &TriangleMesh, &ExcludeMap, &Elements, &HalfRadius, &PositionTargetStiffness, &UseAnisotropicSpring](int32 SurfaceVdx)
			{
				const int32 Index = SurfaceVertices[SurfaceVdx];
				TArray< TTriangleCollisionPoint<FSolverReal>> Result;
				//PointProximityQuery
				if (TriangleMesh.PointClosestTriangleQuery(Spatial, static_cast<const TArrayView<const FSolverVec3>&>(Particles.XArray()), Index, Particles.GetX(Index), HalfRadius, HalfRadius,
					[this, &Elements, &ExcludeMap](const int32 PointIndex, const int32 TriangleIndex)->bool
					{
						return  !(ExcludeMap.Find(PointIndex) && ExcludeMap[PointIndex].Contains(TriangleIndex));
					},
					Result))
				{
					for (const TTriangleCollisionPoint<FSolverReal>& CollisionPoint : Result)
					{
						if (CollisionPoint.Phi < 0)
						{
							const TVector<int32, 3>& Elem = Elements[CollisionPoint.Indices[1]];
							const int32 IndexToWrite = ConstraintIndex.fetch_add(1);

							FGaussSeidelSpringConstraintSingleData SingleConstraintData;
							SingleConstraintData.SingleIndices = { Elem[0], Elem[1] ,Elem[2] };
							SingleConstraintData.SingleSecondIndices =  { Index };
							SingleConstraintData.SingleWeights = { CollisionPoint.Bary[1], CollisionPoint.Bary[2], CollisionPoint.Bary[3] };
							SingleConstraintData.SingleSecondWeights = {FSolverReal(1.f)};
							SingleConstraintData.bIsAnisotropic = UseAnisotropicSpring;
							SingleConstraintData.SingleNormal = CollisionPoint.Normal;
							SingleConstraintData.bIsZeroRestLength = true; //Push-out type collision springs should be zero rest length

							float SpringStiffness = 0.f;
							for (int32 TriLocalIdx = 0; TriLocalIdx < 3; TriLocalIdx++)
							{
								SpringStiffness += SingleConstraintData.SingleWeights[TriLocalIdx] * PositionTargetStiffness * Particles.M(Elem[TriLocalIdx]);
							}
							SpringStiffness += PositionTargetStiffness * Particles.M(Index);
							SingleConstraintData.SingleStiffness = (FSolverReal)SpringStiffness;
							ConstraintsData.SetSingleConstraint(SingleConstraintData, IndexToWrite);
						}
					}
				}
			}
		);

		// Shrink the arrays to the actual number of found constraints.
		const int32 ConstraintNum = ConstraintIndex.load();
		Resize(ConstraintNum);
	}

	// Explicit template instantiation for the only used type
	template void FGaussSeidelSpringConstraints::CollisionDetectionSpatialHash<FTriangleMesh::TSpatialHashType<FSolverReal>>(const FSolverParticles&, const TArray<int32>&, const FTriangleMesh&, const TArray<int32>&, const FTriangleMesh::TSpatialHashType<FSolverReal>&, float, float, bool);
	template void FGaussSeidelSpringConstraints::CollisionDetectionSpatialHashInComponent<FTriangleMesh::TSpatialHashType<FSolverReal>>(const FSolverParticles&, const TArray<int32>&, const FTriangleMesh&, const TMap<int32, TSet<int32>>&, const FTriangleMesh::TSpatialHashType<FSolverReal>&, float, float, bool);

} // namespace Chaos::Softs
