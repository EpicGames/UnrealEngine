// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/XPBDWeakConstraints.h"
#include "Chaos/DebugDrawQueue.h"

namespace Chaos::Softs
{
	using Chaos::TVec3;

	// -------------------------------------------------------------------
	// Constructors (FSolverParticles — delegate to Range versions)
	// -------------------------------------------------------------------

	FXPBDWeightedSpringConstraints::FXPBDWeightedSpringConstraints(
		const FSolverParticles& InParticles,
		const TArray<TArray<int32>>& InIndices,
		const TArray<TArray<FSolverReal>>& InWeights,
		const TArray<FSolverReal>& InStiffness
	)
		: FXPBDWeightedSpringConstraints(
			FSolverParticlesRange(InParticles),
			InIndices, InWeights, InStiffness)
	{}

	FXPBDWeightedSpringConstraints::FXPBDWeightedSpringConstraints(
		const FSolverParticles& InParticles,
		const TArray<TArray<int32>>& InIndices,
		const TArray<TArray<FSolverReal>>& InWeights,
		const TArray<FSolverReal>& InStiffness,
		const TArray<TArray<int32>>& InSecondIndices,
		const TArray<TArray<FSolverReal>>& InSecondWeights,
		const FDeformableXPBDWeightedSpringConstraintParams& InParams
	)
		: FXPBDWeightedSpringConstraints(
			FSolverParticlesRange(InParticles),
			InIndices, InWeights, InStiffness, InSecondIndices, InSecondWeights, InParams)
	{}

	FXPBDWeightedSpringConstraints::FXPBDWeightedSpringConstraints(
		const FSolverParticles& InParticles,
		const FSolverParticles& InSecondParticles,
		const TArray<TArray<int32>>& InIndices,
		const TArray<TArray<FSolverReal>>& InWeights,
		const TArray<FSolverReal>& InStiffness,
		const TArray<TArray<int32>>& InSecondIndices,
		const TArray<TArray<FSolverReal>>& InSecondWeights,
		const FDeformableXPBDWeightedSpringConstraintParams& InParams
	)
		: FXPBDWeightedSpringConstraints(
			FSolverParticlesRange(InParticles),
			FSolverParticlesRange(InSecondParticles),
			InIndices, InWeights, InStiffness, InSecondIndices, InSecondWeights, InParams)
	{}

	// -------------------------------------------------------------------
	// Constructors (FSolverParticlesRange — primary implementations)
	// -------------------------------------------------------------------

	FXPBDWeightedSpringConstraints::FXPBDWeightedSpringConstraints(
		const FSolverParticlesRange& InParticlesRange,
		const TArray<TArray<int32>>& InIndices,
		const TArray<TArray<FSolverReal>>& InWeights,
		const TArray<FSolverReal>& InStiffness
	)
		: Indices(InIndices), Weights(InWeights), Stiffness(InStiffness)
	{
		// Initialize target positions to zero — caller must call UpdateTargets() before Apply() to set actual targets
		Constraints.Init(TVector<FSolverReal, 3>((FSolverReal)0.), Indices.Num());
		InitColor(InParticlesRange);
		LambdaArray.Init((FSolverReal)0., Indices.Num() * 3);
	}

	FXPBDWeightedSpringConstraints::FXPBDWeightedSpringConstraints(
		const FSolverParticlesRange& InParticlesRange,
		const TArray<TArray<int32>>& InIndices,
		const TArray<TArray<FSolverReal>>& InWeights,
		const TArray<FSolverReal>& InStiffness,
		const TArray<TArray<int32>>& InSecondIndices,
		const TArray<TArray<FSolverReal>>& InSecondWeights,
		const FDeformableXPBDWeightedSpringConstraintParams& InParams
	)
		: Indices(InIndices), Weights(InWeights), SecondIndices(InSecondIndices), SecondWeights(InSecondWeights), Stiffness(InStiffness), DebugDrawParams(InParams)
	{
		ensureMsgf(Indices.Num() == SecondIndices.Num(), TEXT("Input Double Bindings have wrong size"));

		for (int32 ConstraintIdx = 0; ConstraintIdx < Indices.Num(); ConstraintIdx++)
		{
			const TSet<int32> IndicesSet = TSet<int32>(Indices[ConstraintIdx]);
			for (int32 SecondLocalIdx = 0; SecondLocalIdx < SecondIndices[ConstraintIdx].Num(); SecondLocalIdx++)
			{
				ensureMsgf(!IndicesSet.Contains(SecondIndices[ConstraintIdx][SecondLocalIdx]), TEXT("Indices and Second Indices overlaps. Currently not supported"));
			}
		}
		InitColor(InParticlesRange);
		LambdaArray.Init((FSolverReal)0., Indices.Num() * 3);
	}

	FXPBDWeightedSpringConstraints::FXPBDWeightedSpringConstraints(
		const FSolverParticlesRange& InParticlesRange,
		const FSolverParticlesRange& InSecondParticlesRange,
		const TArray<TArray<int32>>& InIndices,
		const TArray<TArray<FSolverReal>>& InWeights,
		const TArray<FSolverReal>& InStiffness,
		const TArray<TArray<int32>>& InSecondIndices,
		const TArray<TArray<FSolverReal>>& InSecondWeights,
		const FDeformableXPBDWeightedSpringConstraintParams& InParams
	)
		: Indices(InIndices), Weights(InWeights), SecondIndices(InSecondIndices), SecondWeights(InSecondWeights), Stiffness(InStiffness), DebugDrawParams(InParams)
	{
		ensureMsgf(Indices.Num() == SecondIndices.Num(), TEXT("Input Double Bindings have wrong size"));

		InitColor(InParticlesRange, InSecondParticlesRange);
		LambdaArray.Init((FSolverReal)0., Indices.Num() * 3);
	}

	// -------------------------------------------------------------------
	// Apply methods
	// -------------------------------------------------------------------

	void FXPBDWeightedSpringConstraints::ApplyInParallel(FSolverParticlesRange& Particles, const FSolverReal Dt) const
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(STAT_ChaosXPBDWeakConstraintApply);

		if ((ConstraintsPerColor.Num() > 0))
		{
			if (SecondIndices.Num() == 0)
			{
				for (int32 Color = 0; Color < ConstraintsPerColor.Num(); Color++)
				{
					// TODO: replace with UE::Tasks::FTask with dependencies and minimum batch size control
					PhysicsParallelFor(ConstraintsPerColor[Color].Num(), [&](const int32 Index)
						{
							const int32 ConstraintIndex = ConstraintsPerColor[Color][Index];
							ApplySingleConstraintWithoutSelfTarget(Particles, Dt, ConstraintIndex);
						});
				}
			}
			else
			{
				for (int32 Color = 0; Color < ConstraintsPerColor.Num(); Color++)
				{
					PhysicsParallelFor(ConstraintsPerColor[Color].Num(), [&](const int32 Index)
						{
							const int32 ConstraintIndex = ConstraintsPerColor[Color][Index];
							ApplySingleConstraintWithSelfTarget(Particles, Dt, ConstraintIndex);
						});
				}
			}
		}
	}

	void FXPBDWeightedSpringConstraints::ApplyInParallel(FSolverParticles& Particles, const FSolverReal Dt) const
	{
		FSolverParticlesRange Range(Particles);
		ApplyInParallel(Range, Dt);
	}

	void FXPBDWeightedSpringConstraints::Apply(FSolverParticlesRange& Particles, const FSolverReal Dt) const
	{
		ApplyInParallel(Particles, Dt);
	}

	void FXPBDWeightedSpringConstraints::ApplyInSerial(FSolverParticlesRange& Particles, FSolverParticlesRange& SecondParticles, const FSolverReal Dt) const
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(STAT_ChaosXPBDWeakConstraintApply);

		for (int32 ConstraintIdx = 0; ConstraintIdx < Indices.Num(); ++ConstraintIdx)
		{
			ApplySingleConstraintWithSelfTarget(Particles, SecondParticles, Dt, ConstraintIdx);
		}
	}

	void FXPBDWeightedSpringConstraints::ApplyInParallel(FSolverParticlesRange& Particles, FSolverParticlesRange& SecondParticles, const FSolverReal Dt) const
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(STAT_ChaosXPBDWeakConstraintTwoParticlesApply);
		if ((ConstraintsPerColor.Num() > 0))
		{
			if (ensureMsgf(Indices.Num() == SecondIndices.Num(), TEXT("Constraint source and target have nonequal sizes")))
			{
				for (int32 Color = 0; Color < ConstraintsPerColor.Num(); Color++)
				{
					PhysicsParallelFor(ConstraintsPerColor[Color].Num(), [&](const int32 Index)
						{
							const int32 ConstraintIndex = ConstraintsPerColor[Color][Index];
							ApplySingleConstraintBetweenParticles(Particles, SecondParticles, Dt, ConstraintIndex);
						});
				}
			}
		}
	}

	// -------------------------------------------------------------------
	// Init methods
	// -------------------------------------------------------------------

	void FXPBDWeightedSpringConstraints::Init(const FSolverParticlesRange& InParticles, const FSolverReal Dt) const
	{
		for (FSolverReal& Lambdas : LambdaArray) { Lambdas = (FSolverReal)0.; }
		if (DebugDrawParams.bVisualizeBindings)
		{
			VisualizeAllBindings(InParticles, Dt);
		}
	}

	void FXPBDWeightedSpringConstraints::Init(const FSolverParticles& InParticles, const FSolverReal Dt) const
	{
		Init(FSolverParticlesRange(InParticles), Dt);
	}

	void FXPBDWeightedSpringConstraints::Init(const FSolverParticlesRange& InParticles, const FSolverParticlesRange& InSecondParticles, const FSolverReal Dt) const
	{
		for (FSolverReal& Lambdas : LambdaArray) { Lambdas = (FSolverReal)0.; }
	}

	// -------------------------------------------------------------------
	// ComputeSpringEdge (virtual)
	// -------------------------------------------------------------------

	bool FXPBDWeightedSpringConstraints::ComputeSpringEdge(const FSolverParticlesRange& InParticles, TVec3<FSolverReal>& OutSpringEdge, int32 ConstraintIndex, bool bUseParticleX, bool bUseConstraintTargetPosition) const
	{
		return ComputeSpringEdge(InParticles, InParticles, OutSpringEdge, ConstraintIndex, bUseParticleX, bUseConstraintTargetPosition);
	}

	bool FXPBDWeightedSpringConstraints::ComputeSpringEdge(const FSolverParticlesRange& InParticles, const FSolverParticlesRange& InSecondParticles,
		TVec3<FSolverReal>& OutSpringEdge, int32 ConstraintIndex, bool bUseParticleX, bool bUseConstraintTargetPosition) const
	{
		OutSpringEdge = TVec3<FSolverReal>((FSolverReal)0.);
		if (ensure(Indices[ConstraintIndex].Num() == Weights[ConstraintIndex].Num() && 
			(bUseConstraintTargetPosition || SecondIndices[ConstraintIndex].Num() == SecondWeights[ConstraintIndex].Num())))
		{
			auto GetPosition = [bUseParticleX](const FSolverParticlesRange& InParticles, int32 Pdx) -> const TVec3<FSolverReal>&
				{
					return bUseParticleX ? InParticles.X(Pdx) : InParticles.P(Pdx);
				};
			for (int32 LocalIdx = 0; LocalIdx < Indices[ConstraintIndex].Num(); ++LocalIdx)
			{
				OutSpringEdge += Weights[ConstraintIndex][LocalIdx] * GetPosition(InParticles, Indices[ConstraintIndex][LocalIdx]);
			}
			if (bUseConstraintTargetPosition)
			{
				OutSpringEdge -= Constraints[ConstraintIndex];
			}
			else
			{
				for (int32 SecondLocalIdx = 0; SecondLocalIdx < SecondIndices[ConstraintIndex].Num(); ++SecondLocalIdx)
				{
					OutSpringEdge -= SecondWeights[ConstraintIndex][SecondLocalIdx] * GetPosition(InSecondParticles, SecondIndices[ConstraintIndex][SecondLocalIdx]);
				}
			}
			return true;
		}
		else
		{
			return false;
		}
	}

	// -------------------------------------------------------------------
	// InitColor (private)
	// -------------------------------------------------------------------

	void FXPBDWeightedSpringConstraints::InitColor(const FSolverParticlesRange& ParticlesRange)
	{
		Chaos::ComputeWeakConstraintsColoring(Indices, SecondIndices, ParticlesRange.Size(), ConstraintsPerColor);
	}

	void FXPBDWeightedSpringConstraints::InitColor(const FSolverParticlesRange& ParticlesRange, const FSolverParticlesRange& SecondParticlesRange)
	{
		Chaos::ComputeWeakConstraintsColoring(Indices, SecondIndices, ParticlesRange.Size(), SecondParticlesRange.Size(), ConstraintsPerColor);
	}

	// -------------------------------------------------------------------
	// ApplySingleConstraint variants (private)
	// -------------------------------------------------------------------

	void FXPBDWeightedSpringConstraints::ApplySingleConstraintWithoutSelfTarget(FSolverParticlesRange& Particles, const FSolverReal Dt, const int32 ConstraintIndex, const FSolverReal Tol) const
	{
		if (Stiffness[ConstraintIndex] < Tol || Dt == (FSolverReal)0.)
		{
			return;
		}

		const FSolverReal AlphaTilde = Stiffness[ConstraintIndex] > SoftMaxStiffness ? FSolverReal(2) / (Stiffness[ConstraintIndex] * Dt * Dt) : FSolverReal(0);

		TVec3<FSolverReal> SpringEdge;
		if (!ComputeSpringEdge(Particles, SpringEdge, ConstraintIndex, /*bUseParticleX*/ false, /*bUseConstraintTargetPosition*/ true))
		{
			return;
		}

		FSolverReal Denom = AlphaTilde;
		for (int32 LocalIdx = 0; LocalIdx < Indices[ConstraintIndex].Num(); LocalIdx++)
		{
			Denom += Weights[ConstraintIndex][LocalIdx] * Weights[ConstraintIndex][LocalIdx] * Particles.InvM(Indices[ConstraintIndex][LocalIdx]);
		}

		if (Denom == (FSolverReal)0.)
		{
			return;
		}

		for (int32 Beta = 0; Beta < 3; Beta++)
		{
			const FSolverReal Cj = SpringEdge[Beta];
			FSolverReal DLambda = -Cj - AlphaTilde * LambdaArray[ConstraintIndex * 3 + Beta];

			DLambda /= Denom;
			LambdaArray[ConstraintIndex * 3 + Beta] += DLambda;
			for (int32 LocalIdx = 0; LocalIdx < Indices[ConstraintIndex].Num(); LocalIdx++)
			{
				Particles.P(Indices[ConstraintIndex][LocalIdx])[Beta] += DLambda * Weights[ConstraintIndex][LocalIdx] * Particles.InvM(Indices[ConstraintIndex][LocalIdx]);
			}
		}
	}

	void FXPBDWeightedSpringConstraints::ApplySingleConstraintWithSelfTarget(FSolverParticlesRange& Particles, const FSolverReal Dt, const int32 ConstraintIndex, const FSolverReal Tol) const
	{
		if (Stiffness[ConstraintIndex] < Tol || Dt == (FSolverReal)0. || 
			!ensure(SecondIndices.IsValidIndex(ConstraintIndex) && SecondWeights.IsValidIndex(ConstraintIndex)))
		{
			return;
		}

		const FSolverReal AlphaTilde = Stiffness[ConstraintIndex] > SoftMaxStiffness ? FSolverReal(2) / (Stiffness[ConstraintIndex] * Dt * Dt) : FSolverReal(0);

		TVec3<FSolverReal> SpringEdge;
		if (!ComputeSpringEdge(Particles, SpringEdge, ConstraintIndex, /*bUseParticleX*/ false, /*bUseConstraintTargetPosition*/ false))
		{
			return;
		}

		FSolverReal Denom = AlphaTilde;
		for (int32 LocalIdx = 0; LocalIdx < Indices[ConstraintIndex].Num(); LocalIdx++)
		{
			Denom += Weights[ConstraintIndex][LocalIdx] * Weights[ConstraintIndex][LocalIdx] * Particles.InvM(Indices[ConstraintIndex][LocalIdx]);
		}

		for (int32 SecondLocalIdx = 0; SecondLocalIdx < SecondIndices[ConstraintIndex].Num(); SecondLocalIdx++)
		{
			Denom += SecondWeights[ConstraintIndex][SecondLocalIdx] * SecondWeights[ConstraintIndex][SecondLocalIdx] * Particles.InvM(SecondIndices[ConstraintIndex][SecondLocalIdx]);
		}

		if (Denom == (FSolverReal)0.)
		{
			return;
		}

		for (int32 Beta = 0; Beta < 3; Beta++)
		{
			const FSolverReal Cj = SpringEdge[Beta];
			FSolverReal DLambda = -Cj - AlphaTilde * LambdaArray[ConstraintIndex * 3 + Beta];

			DLambda /= Denom;
			LambdaArray[ConstraintIndex * 3 + Beta] += DLambda;
			for (int32 LocalIdx = 0; LocalIdx < Indices[ConstraintIndex].Num(); LocalIdx++)
			{
				Particles.P(Indices[ConstraintIndex][LocalIdx])[Beta] += DLambda * Weights[ConstraintIndex][LocalIdx] * Particles.InvM(Indices[ConstraintIndex][LocalIdx]);
			}

			for (int32 SecondLocalIdx = 0; SecondLocalIdx < SecondIndices[ConstraintIndex].Num(); SecondLocalIdx++)
			{
				Particles.P(SecondIndices[ConstraintIndex][SecondLocalIdx])[Beta] -= DLambda * SecondWeights[ConstraintIndex][SecondLocalIdx] * Particles.InvM(SecondIndices[ConstraintIndex][SecondLocalIdx]);
			}
		}
	}

	//Deprecating in the next CL, replaced with ApplySingleConstraintBetweenParticles
	void FXPBDWeightedSpringConstraints::ApplySingleConstraintWithSelfTarget(FSolverParticlesRange& Particles, FSolverParticlesRange& SecondParticles, const FSolverReal Dt, const int32 ConstraintIndex, const FSolverReal Tol) const
	{
		if (Stiffness[ConstraintIndex] < Tol)
		{
			return;
		}

		ensure(SecondIndices.Num() > 0);
		FSolverReal AlphaTilde = FSolverReal(2) / (Stiffness[ConstraintIndex] * Dt * Dt);
		if (Stiffness[ConstraintIndex] > SoftMaxStiffness)
		{
			AlphaTilde = FSolverReal(0);
		}

		TVec3<FSolverReal> SpringEdge((FSolverReal)0.);
		for (int32 LocalIdx = 0; LocalIdx < Indices[ConstraintIndex].Num(); LocalIdx++)
		{
			for (int32 Alpha = 0; Alpha < 3; Alpha++)
			{
				SpringEdge[Alpha] += Weights[ConstraintIndex][LocalIdx] * Particles.P(Indices[ConstraintIndex][LocalIdx])[Alpha];
			}
		}
		for (int32 SecondLocalIdx = 0; SecondLocalIdx < SecondIndices[ConstraintIndex].Num(); SecondLocalIdx++)
		{
			SpringEdge -= SecondWeights[ConstraintIndex][SecondLocalIdx] * SecondParticles.P(SecondIndices[ConstraintIndex][SecondLocalIdx]);
		}

		FSolverReal Denom = AlphaTilde;
		for (int32 LocalIdx = 0; LocalIdx < Indices[ConstraintIndex].Num(); LocalIdx++)
		{
			Denom += Weights[ConstraintIndex][LocalIdx] * Weights[ConstraintIndex][LocalIdx] * Particles.InvM(Indices[ConstraintIndex][LocalIdx]);
		}
		for (int32 SecondLocalIdx = 0; SecondLocalIdx < SecondIndices[ConstraintIndex].Num(); SecondLocalIdx++)
		{
			Denom += SecondWeights[ConstraintIndex][SecondLocalIdx] * SecondWeights[ConstraintIndex][SecondLocalIdx] * SecondParticles.InvM(SecondIndices[ConstraintIndex][SecondLocalIdx]);
		}
		if (Denom == (FSolverReal)0.)
		{
			return;
		}
		for (int32 Beta = 0; Beta < 3; Beta++)
		{
			const FSolverReal Cj = SpringEdge[Beta];
			FSolverReal DLambda = -Cj - AlphaTilde * LambdaArray[ConstraintIndex * 3 + Beta];
			DLambda /= Denom;
			LambdaArray[ConstraintIndex * 3 + Beta] += DLambda;
			for (int32 LocalIdx = 0; LocalIdx < Indices[ConstraintIndex].Num(); LocalIdx++)
			{
				Particles.P(Indices[ConstraintIndex][LocalIdx])[Beta] += DLambda * Weights[ConstraintIndex][LocalIdx] * Particles.InvM(Indices[ConstraintIndex][LocalIdx]);
			}
			for (int32 SecondLocalIdx = 0; SecondLocalIdx < SecondIndices[ConstraintIndex].Num(); SecondLocalIdx++)
			{
				SecondParticles.P(SecondIndices[ConstraintIndex][SecondLocalIdx])[Beta] -= DLambda * SecondWeights[ConstraintIndex][SecondLocalIdx] * SecondParticles.InvM(SecondIndices[ConstraintIndex][SecondLocalIdx]);
			}
		}
	}

	void FXPBDWeightedSpringConstraints::ApplySingleConstraintBetweenParticles(FSolverParticlesRange& Particles, FSolverParticlesRange& SecondParticles, const FSolverReal Dt, const int32 ConstraintIndex, const FSolverReal Tol) const
	{
		if (Stiffness[ConstraintIndex] < Tol || Dt == (FSolverReal)0. ||
			!ensure(SecondIndices.IsValidIndex(ConstraintIndex) && SecondWeights.IsValidIndex(ConstraintIndex)))
		{
			return;
		}

		const FSolverReal AlphaTilde = Stiffness[ConstraintIndex] > SoftMaxStiffness ? FSolverReal(2) / (Stiffness[ConstraintIndex] * Dt * Dt) : FSolverReal(0);

		TVec3<FSolverReal> SpringEdge;
		if (!ComputeSpringEdge(Particles, SecondParticles, SpringEdge, ConstraintIndex,
			/*bUseParticleX*/ false, /*bUseConstraintTargetPosition*/ false))
		{
			return;
		}

		FSolverReal Denom = AlphaTilde;
		for (int32 LocalIdx = 0; LocalIdx < Indices[ConstraintIndex].Num(); LocalIdx++)
		{
			Denom += Weights[ConstraintIndex][LocalIdx] * Weights[ConstraintIndex][LocalIdx] * Particles.InvM(Indices[ConstraintIndex][LocalIdx]);
		}
		for (int32 SecondLocalIdx = 0; SecondLocalIdx < SecondIndices[ConstraintIndex].Num(); SecondLocalIdx++)
		{
			Denom += SecondWeights[ConstraintIndex][SecondLocalIdx] * SecondWeights[ConstraintIndex][SecondLocalIdx] * SecondParticles.InvM(SecondIndices[ConstraintIndex][SecondLocalIdx]);
		}
		if (Denom == (FSolverReal)0.)
		{
			return;
		}
		for (int32 Beta = 0; Beta < 3; Beta++)
		{
			const FSolverReal Cj = SpringEdge[Beta];
			FSolverReal DLambda = -Cj - AlphaTilde * LambdaArray[ConstraintIndex * 3 + Beta];
			DLambda /= Denom;
			LambdaArray[ConstraintIndex * 3 + Beta] += DLambda;
			for (int32 LocalIdx = 0; LocalIdx < Indices[ConstraintIndex].Num(); LocalIdx++)
			{
				Particles.P(Indices[ConstraintIndex][LocalIdx])[Beta] += DLambda * Weights[ConstraintIndex][LocalIdx] * Particles.InvM(Indices[ConstraintIndex][LocalIdx]);
			}
			for (int32 SecondLocalIdx = 0; SecondLocalIdx < SecondIndices[ConstraintIndex].Num(); SecondLocalIdx++)
			{
				SecondParticles.P(SecondIndices[ConstraintIndex][SecondLocalIdx])[Beta] -= DLambda * SecondWeights[ConstraintIndex][SecondLocalIdx] * SecondParticles.InvM(SecondIndices[ConstraintIndex][SecondLocalIdx]);
			}
		}
	}

	// -------------------------------------------------------------------
	// VisualizeAllBindings
	// -------------------------------------------------------------------

	void FXPBDWeightedSpringConstraints::VisualizeAllBindings(const FSolverParticlesRange& InParticles, const FSolverParticlesRange& InSecondParticles,
		const TFunctionRef<void(const FVector& Pos0, const FVector& Pos1, const FLinearColor& Color)>& DrawLine,
		const TFunctionRef<void(const FVector& Pos, const FLinearColor& Color)>& DrawPoint) const
	{
#if CHAOS_DEBUG_DRAW
		auto DoubleVert = [](const Chaos::TVec3<FSolverReal>& V) { return FVector3d(V.X, V.Y, V.Z); };
		for (int32 ConstraintIndex = 0; ConstraintIndex < Indices.Num(); ++ConstraintIndex)
		{
			Chaos::TVec3<FSolverReal> SourcePos((FSolverReal)0.), TargetPos((FSolverReal)0.);
			for (int32 IndicesLocalIndex = 0; IndicesLocalIndex < Indices[ConstraintIndex].Num(); ++IndicesLocalIndex)
			{
				SourcePos += Weights[ConstraintIndex][IndicesLocalIndex] * InParticles.P(Indices[ConstraintIndex][IndicesLocalIndex]);
			}
			for (int32 SecondIndicesLocalIndex = 0; SecondIndicesLocalIndex < SecondIndices[ConstraintIndex].Num(); ++SecondIndicesLocalIndex)
			{
				TargetPos += SecondWeights[ConstraintIndex][SecondIndicesLocalIndex] * InSecondParticles.P(SecondIndices[ConstraintIndex][SecondIndicesLocalIndex]);
			}

			if (Indices[ConstraintIndex].Num() > 1)
			{
				for (int32 IndicesLocalIndex = 0; IndicesLocalIndex < Indices[ConstraintIndex].Num(); ++IndicesLocalIndex)
				{
					DrawLine(DoubleVert(InParticles.P(Indices[ConstraintIndex][IndicesLocalIndex])), DoubleVert(InParticles.P(Indices[ConstraintIndex][(IndicesLocalIndex + 1) % Indices[ConstraintIndex].Num()])), FColor::Green);
				}
			}

			if (SecondIndices[ConstraintIndex].Num() > 1)
			{
				for (int32 SecondIndicesLocalIndex = 0; SecondIndicesLocalIndex < SecondIndices[ConstraintIndex].Num(); ++SecondIndicesLocalIndex)
				{
					DrawLine(DoubleVert(InSecondParticles.P(SecondIndices[ConstraintIndex][SecondIndicesLocalIndex])), DoubleVert(InSecondParticles.P(SecondIndices[ConstraintIndex][(SecondIndicesLocalIndex + 1) % SecondIndices[ConstraintIndex].Num()])), FColor::Green);
				}
			}
			DrawPoint(DoubleVert(SourcePos), FColor::Red);
			DrawPoint(DoubleVert(TargetPos), FColor::Red);
			DrawLine(DoubleVert(SourcePos), DoubleVert(TargetPos), FColor::Yellow);
		}
#endif
	}

	void FXPBDWeightedSpringConstraints::VisualizeAllBindings(const FSolverParticlesRange& InParticles, const FSolverReal Dt) const
	{
#if CHAOS_DEBUG_DRAW
		auto DoubleVert = [](const Chaos::TVec3<FSolverReal>& V) { return FVector3d(V.X, V.Y, V.Z); };
		for (int32 ConstraintIndex = 0; ConstraintIndex < Indices.Num(); ++ConstraintIndex)
		{
			Chaos::TVec3<FSolverReal> SourcePos((FSolverReal)0.), TargetPos((FSolverReal)0.);
			for (int32 IndicesLocalIndex = 0; IndicesLocalIndex < Indices[ConstraintIndex].Num(); ++IndicesLocalIndex)
			{
				SourcePos += Weights[ConstraintIndex][IndicesLocalIndex] * InParticles.P(Indices[ConstraintIndex][IndicesLocalIndex]);
			}
			if (SecondIndices.IsValidIndex(ConstraintIndex) && SecondIndices[ConstraintIndex].Num() > 0)
			{
				for (int32 SecondIndicesLocalIndex = 0; SecondIndicesLocalIndex < SecondIndices[ConstraintIndex].Num(); ++SecondIndicesLocalIndex)
				{
					TargetPos += SecondWeights[ConstraintIndex][SecondIndicesLocalIndex] * InParticles.P(SecondIndices[ConstraintIndex][SecondIndicesLocalIndex]);
				}
			}
			else
			{
				if (ensure(Constraints.IsValidIndex(ConstraintIndex)))
				{
					TargetPos = Constraints[ConstraintIndex];
				}
			}

			const float ParticleThickness = DebugDrawParams.DebugParticleWidth;
			const float LineThickness = DebugDrawParams.DebugLineWidth;

			if (Indices[ConstraintIndex].Num() > 1)
			{
				for (int32 IndicesLocalIndex = 0; IndicesLocalIndex < Indices[ConstraintIndex].Num(); ++IndicesLocalIndex)
				{
					Chaos::FDebugDrawQueue::GetInstance().DrawDebugLine(DoubleVert(InParticles.P(Indices[ConstraintIndex][IndicesLocalIndex])), 
						DoubleVert(InParticles.P(Indices[ConstraintIndex][(IndicesLocalIndex + 1) % Indices[ConstraintIndex].Num()])), FColor::Green, false, Dt, 0, LineThickness);
				}
			}

			if (SecondIndices[ConstraintIndex].Num() > 1)
			{
				for (int32 SecondIndicesLocalIndex = 0; SecondIndicesLocalIndex < SecondIndices[ConstraintIndex].Num(); ++SecondIndicesLocalIndex)
				{
					Chaos::FDebugDrawQueue::GetInstance().DrawDebugLine(DoubleVert(InParticles.P(SecondIndices[ConstraintIndex][SecondIndicesLocalIndex])), 
						DoubleVert(InParticles.P(SecondIndices[ConstraintIndex][(SecondIndicesLocalIndex + 1) % SecondIndices[ConstraintIndex].Num()])), FColor::Green, false, Dt, 0, LineThickness);
				}
			}

			Chaos::FDebugDrawQueue::GetInstance().DrawDebugPoint(DoubleVert(SourcePos), FColor::Red, false, Dt, 0, ParticleThickness);
			Chaos::FDebugDrawQueue::GetInstance().DrawDebugPoint(DoubleVert(TargetPos), FColor::Red, false, Dt, 0, ParticleThickness);
			Chaos::FDebugDrawQueue::GetInstance().DrawDebugLine(DoubleVert(SourcePos), DoubleVert(TargetPos), FColor::Yellow, false, Dt, 0, LineThickness);
		}
#endif
	}

	void FXPBDWeightedSpringConstraints::VisualizeAllBindings(const FSolverParticlesRange& InParticles,
		const TFunctionRef<void(const FVector& Pos0, const FVector& Pos1, const FLinearColor& Color)>& DrawLine,
		const TFunctionRef<void(const FVector& Pos, const FLinearColor& Color)>& DrawPoint) const
	{
		VisualizeAllBindings(InParticles, InParticles, DrawLine, DrawPoint);
	}

} // End namespace Chaos::Softs
