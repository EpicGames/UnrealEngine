// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/Evolution/ConstraintGroupSolver.h"
#include "Chaos/ConstraintHandle.h"
#include "Chaos/Evolution/SolverConstraintContainer.h"
#include "Chaos/PBDConstraintContainer.h"
#include "Tasks/Task.h"

namespace Chaos
{
	namespace CVars
	{
		extern int32 Chaos_SolverPartitionManager_Partitioning;
		extern int32 Chaos_SolverPartitionManager_ParallelSolve;
		extern int32 Chaos_SolverPartitionManager_MinBatchSize;
	}

	namespace Private
	{
		FPBDConstraintGroupSolver::FPBDConstraintGroupSolver()
			: TotalNumConstraints(0)
			, MaxSerialBatches(0)
		{
		}

		FPBDConstraintGroupSolver::~FPBDConstraintGroupSolver()
		{
		}

		void FPBDConstraintGroupSolver::SetConstraintSolver(const int32 ContainerId, TUniquePtr<FConstraintContainerSolver>&& Solver)
		{
			if (ContainerId >= ConstraintContainerSolvers.Num())
			{
				ConstraintContainerSolvers.SetNum(ContainerId + 1);
			}

			ConstraintContainerSolvers[ContainerId] = MoveTemp(Solver);

			SetConstraintSolverImpl(ContainerId);

			SortSolverContainers();
		}

		void FPBDConstraintGroupSolver::SetConstraintSolverPriority(const int32 ContainerId, const int32 Priority)
		{
			if (ConstraintContainerSolvers[ContainerId] != nullptr)
			{
				if (ConstraintContainerSolvers[ContainerId]->GetPriority() != Priority)
				{
					ConstraintContainerSolvers[ContainerId]->SetPriority(Priority);
					SortSolverContainers();
				}
			}
		}

		void FPBDConstraintGroupSolver::SortSolverContainers()
		{
			PrioritizedConstraintContainerSolvers.Reset(ConstraintContainerSolvers.Num());
			for (TUniquePtr<FConstraintContainerSolver>& SolverContainer : ConstraintContainerSolvers)
			{
				if (SolverContainer != nullptr)
				{
					PrioritizedConstraintContainerSolvers.Add(SolverContainer.Get());
				}
			}

			PrioritizedConstraintContainerSolvers.StableSort(
				[](const FConstraintContainerSolver& L, const FConstraintContainerSolver& R)
				{
					return L.GetPriority() < R.GetPriority();
				});
		}

		void FPBDConstraintGroupSolver::Reset()
		{
			SolverBodyContainer.Reset(0);
			ConstraintSolvePrerequisites.Reset(0);

			for (int32 ContainerIndex = 0; ContainerIndex < ConstraintContainerSolvers.Num(); ++ContainerIndex)
			{
				if (ConstraintContainerSolvers[ContainerIndex] != nullptr)
				{
					ConstraintContainerSolvers[ContainerIndex]->Reset(0);
				}
			}

			TotalNumConstraints = 0;
			MaxSerialBatches = 0;

			ResetImpl();
		}

		void FPBDConstraintGroupSolver::AddConstraintsAndBodies()
		{
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_Evolution_AddConstraints);
				AddConstraintsImpl();
			}

			for (int32 ContainerIndex = 0; ContainerIndex < ConstraintContainerSolvers.Num(); ++ContainerIndex)
			{
				if (ConstraintContainerSolvers[ContainerIndex] != nullptr)
				{
					if (CVars::Chaos_SolverPartitionManager_Partitioning)
					{
						QUICK_SCOPE_CYCLE_COUNTER(STAT_Evolution_PrepareSolverPartitions);
						MaxSerialBatches = FMath::Max(MaxSerialBatches, ConstraintContainerSolvers[ContainerIndex]->PrepareSolverPartitions());
					}

					{
						QUICK_SCOPE_CYCLE_COUNTER(STAT_Evolution_AddBodies);
						ConstraintContainerSolvers[ContainerIndex]->AddBodies(SolverBodyContainer);
					}

					TotalNumConstraints += ConstraintContainerSolvers[ContainerIndex]->GetNumConstraints();
				}
			}

			SolverBodyContainer.Lock();
		}

		void FPBDConstraintGroupSolver::GatherBodies(const FReal Dt)
		{
			SolverBodyContainer.GatherInput(Dt, 0, SolverBodyContainer.Num());
		}

		void FPBDConstraintGroupSolver::GatherConstraints(const FReal Dt)
		{
			for (int32 ContainerIndex = 0; ContainerIndex < ConstraintContainerSolvers.Num(); ++ContainerIndex)
			{
				if (ConstraintContainerSolvers[ContainerIndex] != nullptr)
				{
					ConstraintContainerSolvers[ContainerIndex]->GatherInput(Dt);
				}
			}
		}
		void FPBDConstraintGroupSolver::GatherBodies(const FReal Dt, const int32 BeginIndex, const int32 EndIndex)
		{
			SolverBodyContainer.GatherInput(Dt, BeginIndex, EndIndex);

			// Allow derived class to set up some extra data (level, color, etc)
			GatherBodiesImpl(Dt, BeginIndex, EndIndex);
		}

		template<typename LambdaType>
		void FPBDConstraintGroupSolver::ApplyToConstraintRange(const int32 BeginConstraintIndex, const int32 EndConstraintIndex, const LambdaType& Lambda)
		{
			// Loop over all the solver containers until we find the one that contains BeginConstraintIndex
			int32 BeginIndex = BeginConstraintIndex;
			int32 EndIndex = EndConstraintIndex;
			for (int32 ContainerIndex = 0; ContainerIndex < ConstraintContainerSolvers.Num(); ++ContainerIndex)
			{
				if (FConstraintContainerSolver* ConstraintSolver = ConstraintContainerSolvers[ContainerIndex].Get())
				{
					const int32 NumSolverConstraints = ConstraintSolver->GetNumConstraints();
					if (BeginIndex < NumSolverConstraints)
					{
						// The current range start is in this container. 
						// Calculate how many of the constraints we should process and call the lambda
						const int32 BeginSolverIndex = BeginIndex;
						const int32 EndSolverIndex = FMath::Min(EndIndex, NumSolverConstraints);

						Lambda(ConstraintSolver, BeginSolverIndex, EndSolverIndex);
					}

					// Remove the constraints we just processed from the range. The range begin end is now relative to the next container (or empty)
					BeginIndex = FMath::Max(0, BeginIndex - NumSolverConstraints);
					EndIndex = FMath::Max(0, EndIndex - NumSolverConstraints);
					if (EndIndex <= 0)
					{
						break;
					}
				}
			}

			// We should have processed the whole range
			check(EndIndex == 0);
		}


		void FPBDConstraintGroupSolver::GatherConstraints(const FReal Dt, const int32 BeginConstraintIndex, const int32 EndConstraintIndex)
		{
			ApplyToConstraintRange(BeginConstraintIndex, EndConstraintIndex,
				[this, Dt](FConstraintContainerSolver* ConstraintSolver, const int32 BeginSolverIndex, const int32 EndSolverIndex)
				{
					ConstraintSolver->GatherInput(Dt, BeginSolverIndex, EndSolverIndex);
				});
		}

		void FPBDConstraintGroupSolver::ScatterBodies(const FReal Dt)
		{
			SolverBodyContainer.ScatterOutput(0, SolverBodyContainer.Num());
		}

		void FPBDConstraintGroupSolver::ScatterConstraints(const FReal Dt)
		{
			for (int32 ContainerIndex = 0; ContainerIndex < ConstraintContainerSolvers.Num(); ++ContainerIndex)
			{
				if (ConstraintContainerSolvers[ContainerIndex] != nullptr)
				{
					ConstraintContainerSolvers[ContainerIndex]->ScatterOutput(Dt);
				}
			}
		}

		void FPBDConstraintGroupSolver::ScatterBodies(const FReal Dt, const int32 BeginBodyIndex, const int32 EndBodyIndex)
		{
			SolverBodyContainer.ScatterOutput(BeginBodyIndex, EndBodyIndex);
		}

		void FPBDConstraintGroupSolver::ScatterConstraints(const FReal Dt, const int32 BeginConstraintIndex, const int32 EndConstraintIndex)
		{
			ApplyToConstraintRange(BeginConstraintIndex, EndConstraintIndex,
				[this, Dt](FConstraintContainerSolver* ConstraintSolver, const int32 BeginSolverIndex, const int32 EndSolverIndex)
				{
					ConstraintSolver->ScatterOutput(Dt, BeginSolverIndex, EndSolverIndex);
				});
		}

		const bool FPBDConstraintGroupSolver::SolveInParallel() const
		{
			// Solve in parallel if (a) explicitly requested by the user or (b) partitioning triggers it
			const bool bUserTriggeredParallelSolve = CVars::Chaos_SolverPartitionManager_ParallelSolve == 1;
			const bool bPartitioningTriggeredParallelSolve = CVars::Chaos_SolverPartitionManager_ParallelSolve == -1 && CVars::Chaos_SolverPartitionManager_Partitioning > 0;
			return bUserTriggeredParallelSolve || bPartitioningTriggeredParallelSolve;
		}

		int32 FPBDConstraintGroupSolver::GetNumSerialConstraintBatches(const int32 ContainerId) const
		{
			if (ConstraintContainerSolvers.IsValidIndex(ContainerId) && ConstraintContainerSolvers[ContainerId].IsValid())
			{
				return ConstraintContainerSolvers[ContainerId]->GetConstraintsBatchIndices().Num();
			}
			return 0;
		}

		template<typename Lambda>
		void FPBDConstraintGroupSolver::DispatchUpdateSolverBodies(Lambda UpdateSolverBodies, const FReal Dt)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_ConstraintGroupSolver_DispatchUpdateSolverBodies);

			// Blocking serial call
			if (!SolveInParallel())
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_ConstraintGroupSolver_UpdateSolverBodies);
				UpdateSolverBodies(Dt, 0, SolverBodyContainer.Num());
			}
			else // Parallel task graph call with prerequisites
			{
				const int32 MinBodiesPerTask = CVars::Chaos_SolverPartitionManager_MinBatchSize;
				const int32 NumWorkerThreads = (FApp::ShouldUseThreadingForPerformance() && !bSingleWorkerPhysics) ? FMath::Min(FTaskGraphInterface::Get().GetNumWorkerThreads(), Chaos::MaxNumWorkers) : 1;
				const int32 BodiesByTask = FMath::Max(FMath::DivideAndRoundUp(SolverBodyContainer.Num(), NumWorkerThreads), MinBodiesPerTask);
				const int32 NumBodyBatches = FMath::Max(FMath::DivideAndRoundUp(SolverBodyContainer.Num(), BodiesByTask), 1);

				TArray<UE::Tasks::FTask> CurrentTasks;
				CurrentTasks.Reserve(NumBodyBatches);

				for (int32 BatchIndex = 0; BatchIndex < NumBodyBatches; BatchIndex++)
				{
					const int32 BatchStartIndex = BatchIndex * BodiesByTask;
					int32 BatchEndIndex = (BatchIndex + 1) * BodiesByTask;
					BatchEndIndex = FMath::Min(SolverBodyContainer.Num(), BatchEndIndex);
					check(!ConstraintSolvePrerequisites.IsEmpty());
					UE::Tasks::FTask SolverBodyTask = UE::Tasks::Launch(UE_SOURCE_LOCATION,
						[UpdateSolverBodies, Dt, BatchStartIndex, BatchEndIndex]()
						{
							QUICK_SCOPE_CYCLE_COUNTER(STAT_ConstraintGroupSolver_UpdateSolverBodies);
							UpdateSolverBodies(Dt, BatchStartIndex, BatchEndIndex);
						},
						ConstraintSolvePrerequisites.Last(),
						LowLevelTasks::ETaskPriority::High);
					CurrentTasks.Add(SolverBodyTask);
				}

				ConstraintSolvePrerequisites.Add(MoveTemp(CurrentTasks));
			}
		}

		template<typename Lambda>
		void FPBDConstraintGroupSolver::DispatchApplyConstraints(Lambda ApplyConstraints, const FReal Dt, const int32 It, const int32 NumIts)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_ConstraintGroupSolver_DispatchApplyConstraints);

			// NOTE: We loop over prioritized solvers here
			for (int32 ContainerIndex = 0; ContainerIndex < PrioritizedConstraintContainerSolvers.Num(); ++ContainerIndex)
			{
				// Only launch tasks if there are any constraints in the container.
				if (PrioritizedConstraintContainerSolvers[ContainerIndex] != nullptr && PrioritizedConstraintContainerSolvers[ContainerIndex]->GetNumConstraints() > 0)
				{
					const int32 NumConstraints = PrioritizedConstraintContainerSolvers[ContainerIndex]->GetNumConstraints();
					// Blocking serial call
					if (!SolveInParallel())
					{
						QUICK_SCOPE_CYCLE_COUNTER(STAT_ConstraintGroupSolver_ApplyConstraints);
						ApplyConstraints(ContainerIndex, Dt, It, NumIts, 0, NumConstraints);
					}
					else // Parallel task graph call with prerequisites
					{
						const TArray<TArray<int32>>& BatchStartIndices = PrioritizedConstraintContainerSolvers[ContainerIndex]->GetConstraintsBatchIndices();
						const int32 NumSerialBatches = BatchStartIndices.Num();

						// NOTE: If the parallel mode is enabled, there might be a constraint container solver with multiple partitions launching tasks.
						// Therefore, we need to taskify all constraint container solvers to respect the solve order.
						if (NumSerialBatches == 0)
						{
							check(!ConstraintSolvePrerequisites.IsEmpty());
							UE::Tasks::FTask ConstraintSolverTask = UE::Tasks::Launch(UE_SOURCE_LOCATION,
								[ApplyConstraints, ContainerIndex, Dt, It, NumIts, NumConstraints]()
								{
									QUICK_SCOPE_CYCLE_COUNTER(STAT_ConstraintGroupSolver_ApplyConstraints);
									ApplyConstraints(ContainerIndex, Dt, It, NumIts, 0, NumConstraints);
								},
								ConstraintSolvePrerequisites.Last(),
								LowLevelTasks::ETaskPriority::High);
							ConstraintSolvePrerequisites.Add({ ConstraintSolverTask });
						}
						else
						{
							// The constraint solver will launch tasks for the batches as follows:
							// Serial batch 0:
							//    Parallel batch 0.0: ConstraintBatchIndices[0][0] - ConstraintBatchIndices[0][1]
							//    Parallel batch 0.1: ConstraintBatchIndices[0][1] - ConstraintBatchIndices[0][2]
							//	  ...
							//    Parallel batch 0.j: ConstraintBatchIndices[0][j] - ConstraintBatchIndices[0][j+1]
							//	  ...
							//    Parallel batches 0.0 - 0.n need to finish before executing serial batch 1.
							// ...
							// Serial batch i:
							//    Parallel batch i.0: ConstraintBatchIndices[i][0] - ConstraintBatchIndices[i][1]
							//    Parallel batch i.1: ConstraintBatchIndices[i][1] - ConstraintBatchIndices[i][2]
							//	  ...
							//    Parallel batch i.j: ConstraintBatchIndices[i][j] - ConstraintBatchIndices[i][j+1]
							//	  ...
							//    Parallel batches i.0 - i.n need to finish before executing serial batch i+1.
							for (int32 SerialIndex = 0; SerialIndex < BatchStartIndices.Num(); ++SerialIndex)
							{
								check(!BatchStartIndices[SerialIndex].IsEmpty());

								const int32 SerialStartIndex = BatchStartIndices[SerialIndex][0];
								const int32 SerialEndIndex = SerialIndex < (BatchStartIndices.Num() - 1) ? (BatchStartIndices[SerialIndex + 1][0]) : (NumConstraints);
								const TArray<int32>& ParallelStartIndices = BatchStartIndices[SerialIndex];

								if (ensure(!ParallelStartIndices.IsEmpty()))
								{
									TArray<UE::Tasks::FTask> ParallelTasks;
									ParallelTasks.Reserve(ParallelStartIndices.Num());

									for (int32 ParallelIndex = 0; ParallelIndex < ParallelStartIndices.Num(); ++ParallelIndex)
									{
										const int32 ParallelStartIndex = ParallelStartIndices[ParallelIndex];
										const int32 ParallelEndIndex = ParallelIndex < (ParallelStartIndices.Num() - 1) ? (ParallelStartIndices[ParallelIndex + 1]) :
											(SerialIndex < (BatchStartIndices.Num() - 1) ? (BatchStartIndices[SerialIndex + 1][0]) : (NumConstraints));

										check(SerialStartIndex <= ParallelStartIndex);
										check(SerialEndIndex >= ParallelEndIndex);

										check(!ConstraintSolvePrerequisites.IsEmpty());
										UE::Tasks::FTask ConstraintSolverTask = UE::Tasks::Launch(UE_SOURCE_LOCATION,
											[ApplyConstraints, ContainerIndex, Dt, It, NumIts, ParallelStartIndex, ParallelEndIndex]()
											{
												QUICK_SCOPE_CYCLE_COUNTER(STAT_ConstraintGroupSolver_ApplyConstraints);
												ApplyConstraints(ContainerIndex, Dt, It, NumIts, ParallelStartIndex, ParallelEndIndex);
											},
											ConstraintSolvePrerequisites.Last(),
											LowLevelTasks::ETaskPriority::High);
										ParallelTasks.Add(ConstraintSolverTask);
									}

									// All parallel tasks become the prerequisite for the next batch of parallel tasks
									ConstraintSolvePrerequisites.Add(MoveTemp(ParallelTasks));
								}
							}
						}
					}
				}
			}
		}

		void FPBDConstraintGroupSolver::PreApplyPositionConstraintsParallelTasks(const FReal Dt)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_ConstraintGroupSolver_PreApplyPositionConstraintsParallelTasks);

			DispatchApplyConstraints([this](const int32 ContainerIndex, const FReal Dt, const int32 It, const int32 NumIts, const int32 StartIndex, const int32 EndIndex)
				{
					PrioritizedConstraintContainerSolvers[ContainerIndex]->PreApplyPositionConstraints(Dt, StartIndex, EndIndex);
				},
				Dt,
				0,
				0);
		}

		void FPBDConstraintGroupSolver::PreApplyPositionConstraints(const FReal Dt)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_ConstraintGroupSolver_PreApplyPositionConstraints);

			for (int32 ContainerIndex = 0; ContainerIndex < ConstraintContainerSolvers.Num(); ++ContainerIndex)
			{
				if (ConstraintContainerSolvers[ContainerIndex] != nullptr)
				{
					ConstraintContainerSolvers[ContainerIndex]->PreApplyPositionConstraints(Dt);
				}
			}
		}

		void FPBDConstraintGroupSolver::ApplyPositionConstraintsParallelTasks(const FReal Dt)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_ConstraintGroupSolver_ApplyPositionConstraintsParallelTasks);

			const int32 NumIts = Iterations.GetNumPositionIterations();

			for (int32 It = 0; It < NumIts; ++It)
			{
				DispatchApplyConstraints([this](const int32 ContainerIndex, const FReal Dt, const int32 It, const int32 NumIts, const int32 StartIndex, const int32 EndIndex)
					{
						PrioritizedConstraintContainerSolvers[ContainerIndex]->ApplyPositionConstraints(Dt, It, NumIts, StartIndex, EndIndex);
					},
					Dt,
					It,
					NumIts);
			}
		}

		void FPBDConstraintGroupSolver::ApplyPositionConstraints(const FReal Dt)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_ConstraintGroupSolver_ApplyPositionConstraints);

			const int32 NumIts = Iterations.GetNumPositionIterations();

			// NOTE: We loop over prioritized solvers here
			for (int32 It = 0; It < NumIts; ++It)
			{
				for (int32 ContainerIndex = 0; ContainerIndex < PrioritizedConstraintContainerSolvers.Num(); ++ContainerIndex)
				{
					if (PrioritizedConstraintContainerSolvers[ContainerIndex] != nullptr)
					{
						PrioritizedConstraintContainerSolvers[ContainerIndex]->ApplyPositionConstraints(Dt, It, NumIts);
					}
				}
			}
		}

		void FPBDConstraintGroupSolver::PreApplyVelocityConstraintsParallelTasks(const FReal Dt)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_ConstraintGroupSolver_PreApplyVelocityConstraintsParallelTasks);

			// Calculate the velocity from the net change in position after applying position constraints
			DispatchUpdateSolverBodies([this](const FReal Dt, const int32 StartIndex, const int32 EndIndex)
				{
					for (int32 SolverBodyIndex = StartIndex; SolverBodyIndex < EndIndex; ++SolverBodyIndex)
					{
						FSolverBody& SolverBody = SolverBodyContainer.GetSolverBody(SolverBodyIndex);
						SolverBody.SetImplicitVelocity(Dt);
					}
				},
				Dt);

			DispatchApplyConstraints([this](const int32 ContainerIndex, const FReal Dt, const int32 It, const int32 NumIts, const int32 StartIndex, const int32 EndIndex)
				{
					PrioritizedConstraintContainerSolvers[ContainerIndex]->PreApplyVelocityConstraints(Dt, StartIndex, EndIndex);
				},
				Dt,
				0,
				0);
		}

		void FPBDConstraintGroupSolver::PreApplyVelocityConstraints(const FReal Dt)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_ConstraintGroupSolver_PreApplyVelocityConstraints);

			// Calculate the velocity from the net change in position after applying position constraints
			SolverBodyContainer.SetImplicitVelocities(Dt);

			for (int32 ContainerIndex = 0; ContainerIndex < ConstraintContainerSolvers.Num(); ++ContainerIndex)
			{
				if (ConstraintContainerSolvers[ContainerIndex] != nullptr)
				{
					ConstraintContainerSolvers[ContainerIndex]->PreApplyVelocityConstraints(Dt);
				}
			}
		}

		void FPBDConstraintGroupSolver::ApplyVelocityConstraintsParallelTasks(const FReal Dt)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_ConstraintGroupSolver_ApplyVelocityConstraintsParallelTasks);

			const int32 NumIts = Iterations.GetNumVelocityIterations();

			for (int32 It = 0; It < NumIts; ++It)
			{
				DispatchApplyConstraints([this](const int32 ContainerIndex, const FReal Dt, const int32 It, const int32 NumIts, const int32 StartIndex, const int32 EndIndex)
					{
						PrioritizedConstraintContainerSolvers[ContainerIndex]->ApplyVelocityConstraints(Dt, It, NumIts, StartIndex, EndIndex);
					},
					Dt,
					It,
					NumIts);
			}
		}

		void FPBDConstraintGroupSolver::ApplyVelocityConstraints(const FReal Dt)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_ConstraintGroupSolver_ApplyVelocityConstraints);

			const int32 NumIts = Iterations.GetNumVelocityIterations();

			// NOTE: We loop over prioritized solvers here
			for (int32 It = 0; It < NumIts; ++It)
			{
				for (int32 ContainerIndex = 0; ContainerIndex < PrioritizedConstraintContainerSolvers.Num(); ++ContainerIndex)
				{
					if (PrioritizedConstraintContainerSolvers[ContainerIndex] != nullptr)
					{
						PrioritizedConstraintContainerSolvers[ContainerIndex]->ApplyVelocityConstraints(Dt, It, NumIts);
					}
				}
			}
		}

		void FPBDConstraintGroupSolver::PreApplyProjectionConstraintsParallelTasks(const FReal Dt)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_ConstraintGroupSolver_PreApplyProjectionConstraintsParallelTasks);

			// Update the body transforms from the deltas calculated in the constraint solve phases 1 and 2
			// NOTE: deliberately not updating the world-space inertia as it is not used by joint projection
			// and no other constraints currently implement projection
			DispatchUpdateSolverBodies([this](const FReal Dt, const int32 StartIndex, const int32 EndIndex)
				{
					for (int32 SolverBodyIndex = StartIndex; SolverBodyIndex < EndIndex; ++SolverBodyIndex)
					{
						FSolverBody& SolverBody = SolverBodyContainer.GetSolverBody(SolverBodyIndex);
						SolverBody.ApplyCorrections();
					}
				},
				Dt);

			DispatchApplyConstraints([this](const int32 ContainerIndex, const FReal Dt, const int32 It, const int32 NumIts, const int32 StartIndex, const int32 EndIndex)
				{
					PrioritizedConstraintContainerSolvers[ContainerIndex]->PreApplyProjectionConstraints(Dt, StartIndex, EndIndex);
				},
				Dt,
				0,
				0);
		}

		void FPBDConstraintGroupSolver::PreApplyProjectionConstraints(const FReal Dt)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_ConstraintGroupSolver_PreApplyProjectionConstraints);

			// Update the body transforms from the deltas calculated in the constraint solve phases 1 and 2
			// NOTE: deliberately not updating the world-space inertia as it is not used by joint projection
			// and no other constraints currently implement projection
			SolverBodyContainer.ApplyCorrections();

			for (int32 ContainerIndex = 0; ContainerIndex < ConstraintContainerSolvers.Num(); ++ContainerIndex)
			{
				if (ConstraintContainerSolvers[ContainerIndex] != nullptr)
				{
					ConstraintContainerSolvers[ContainerIndex]->PreApplyProjectionConstraints(Dt);
				}
			}
		}

		void FPBDConstraintGroupSolver::ApplyProjectionConstraintsParallelTasks(const FReal Dt)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_ConstraintGroupSolver_ApplyProjectionConstraintsParallelTasks);

			const int32 NumIts = Iterations.GetNumProjectionIterations();

			for (int32 It = 0; It < NumIts; ++It)
			{
				DispatchApplyConstraints([this](const int32 ContainerIndex, const FReal Dt, const int32 It, const int32 NumIts, const int32 StartIndex, const int32 EndIndex)
					{
						PrioritizedConstraintContainerSolvers[ContainerIndex]->ApplyProjectionConstraints(Dt, It, NumIts, StartIndex, EndIndex);
					},
					Dt,
					It,
					NumIts);
			}
		}

		void FPBDConstraintGroupSolver::ApplyProjectionConstraints(const FReal Dt)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_ConstraintGroupSolver_ApplyProjectionConstraints);

			const int32 NumIts = Iterations.GetNumProjectionIterations();

			// NOTE: We loop over prioritized solvers here
			for (int32 It = 0; It < NumIts; ++It)
			{
				for (int32 ContainerIndex = 0; ContainerIndex < PrioritizedConstraintContainerSolvers.Num(); ++ContainerIndex)
				{
					if (PrioritizedConstraintContainerSolvers[ContainerIndex] != nullptr)
					{
						PrioritizedConstraintContainerSolvers[ContainerIndex]->ApplyProjectionConstraints(Dt, It, NumIts);
					}
				}
			}
		}

		TArray<UE::Tasks::FTask> FPBDConstraintGroupSolver::ApplyConstraintsParallelTasks(const FReal Dt)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_ConstraintGroupSolver_ApplyConstraintsParallelTasks);

			// Each element of ConstraintSolvePrerequisites contains an array of all tasks which need to be finished before executing the next task.
			// Only parallelizable batches in PrioritizedConstraintContainerSolvers[SolverIndex]->GetConstraintsBatchIndices()[SerialIndex] can be executed simultaneously.
			// Iterations (It), constraint container solvers (S), serial batches (B) need to be executed in series.
			// Recursive-style dependency:
			//		ConstraintSolvePrerequisites(It0, S0, B0) = empty
			//		ConstraintSolvePrerequisites(It0, S0, B1) = ParallelTasks(It0, S0, B0) = ConstraintSolvePrerequisites(It0, S0, B0) + ParallelTasks(It0, S0, B0)
			//		ConstraintSolvePrerequisites(It0, S0, B2) = ParallelTasks(It0, S0, B0) + ParallelTasks(It0, S0, B1) = ConstraintSolvePrerequisites(It0, S0, B0) + ParallelTasks(It0, S0, B1)
			//      ...
			//		ConstraintSolvePrerequisites(It2, S3, B5) = ConstraintSolvePrerequisites(It2, S3, B4) + ParallelTasks(It2, S3, B5)
			
			// NumIts = sum of iteration count of each constraint type + 3 for pre-apply + 2 for updating solver bodies.
			const int32 NumIts = FMath::Max(Iterations.GetNumPositionIterations() + Iterations.GetNumVelocityIterations() + Iterations.GetNumProjectionIterations() + 3 + 2, 0);
			const int32 NumSolvers = ConstraintContainerSolvers.Num();
			ConstraintSolvePrerequisites.Reserve(NumIts * NumSolvers * MaxSerialBatches + 1);

			// Add an empty task as a prerequisite before launching any tasks
			if (ConstraintSolvePrerequisites.IsEmpty())
			{
				ConstraintSolvePrerequisites.Add({});
			}

			PreApplyPositionConstraintsParallelTasks(Dt);
			ApplyPositionConstraintsParallelTasks(Dt);

			PreApplyVelocityConstraintsParallelTasks(Dt);
			ApplyVelocityConstraintsParallelTasks(Dt);

			PreApplyProjectionConstraintsParallelTasks(Dt);
			ApplyProjectionConstraintsParallelTasks(Dt);

			return ConstraintSolvePrerequisites.Last();
		}

		//
		//
		//
		//
		//

		void FPBDSceneConstraintGroupSolver::AddConstraintsImpl()
		{
			// The Scene Solver solves all constraints, so just add everything
			for (int32 ContainerIndex = 0; ContainerIndex < ConstraintContainerSolvers.Num(); ++ContainerIndex)
			{
				if (ConstraintContainerSolvers[ContainerIndex] != nullptr)
				{
					ConstraintContainerSolvers[ContainerIndex]->AddConstraints();
				}
			}
		}

	}	// namespace Private
}	// namespace Chaos
