// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/Evolution/PBDMinEvolution.h"
#include "Chaos/Collision/BasicCollisionDetector.h"
#include "Chaos/Evolution/SolverConstraintContainer.h"
#include "Chaos/PBDCollisionConstraints.h"
#include "Chaos/PBDConstraintContainer.h"
#include "Chaos/PBDRigidsSOAs.h"
#include "Chaos/PerParticleAddImpulses.h"
#include "Chaos/PerParticleEtherDrag.h"
#include "Chaos/PerParticleEulerStepVelocity.h"
#include "Chaos/PerParticleGravity.h"
#include "Chaos/PerParticleInitForce.h"
#include "Chaos/PerParticlePBDEulerStep.h"
#include "Chaos/PerParticlePBDGroundConstraint.h"
#include "Chaos/PerParticlePBDUpdateFromDeltaPosition.h"
#include "ChaosStats.h"
#include "Chaos/PBDJointConstraints.h"
#include "ChaosVisualDebugger/ChaosVisualDebuggerTrace.h"

//UE_DISABLE_OPTIMIZATION

namespace Chaos
{
#if UE_BUILD_SHIPPING || UE_BUILD_TEST
	CHAOS_API DECLARE_LOG_CATEGORY_EXTERN(LogChaosMinEvolution, Warning, Warning);
#else
	CHAOS_API DECLARE_LOG_CATEGORY_EXTERN(LogChaosMinEvolution, Log, All);
#endif
	DEFINE_LOG_CATEGORY(LogChaosMinEvolution);

	DECLARE_CYCLE_STAT(TEXT("MinEvolution::Advance"), STAT_MinEvolution_Advance, STATGROUP_ChaosMinEvolution);
	DECLARE_CYCLE_STAT(TEXT("MinEvolution::PrepareTick"), STAT_MinEvolution_PrepareTick, STATGROUP_ChaosMinEvolution);
	DECLARE_CYCLE_STAT(TEXT("MinEvolution::UnprepareTick"), STAT_MinEvolution_UnprepareTick, STATGROUP_ChaosMinEvolution);
	DECLARE_CYCLE_STAT(TEXT("MinEvolution::Rewind"), STAT_MinEvolution_Rewind, STATGROUP_ChaosMinEvolution);
	DECLARE_CYCLE_STAT(TEXT("MinEvolution::AdvanceOneTimeStep"), STAT_MinEvolution_AdvanceOneTimeStep, STATGROUP_ChaosMinEvolution);
	DECLARE_CYCLE_STAT(TEXT("MinEvolution::Integrate"), STAT_MinEvolution_Integrate, STATGROUP_ChaosMinEvolution);
	DECLARE_CYCLE_STAT(TEXT("MinEvolution::KinematicTargets"), STAT_MinEvolution_KinematicTargets, STATGROUP_ChaosMinEvolution);
	DECLARE_CYCLE_STAT(TEXT("MinEvolution::Gather"), STAT_MinEvolution_Gather, STATGROUP_ChaosMinEvolution);
	DECLARE_CYCLE_STAT(TEXT("MinEvolution::Scatter"), STAT_MinEvolution_Scatter, STATGROUP_ChaosMinEvolution);
	DECLARE_CYCLE_STAT(TEXT("MinEvolution::ApplyConstraintsPhase1"), STAT_MinEvolution_ApplyConstraintsPhase1, STATGROUP_ChaosMinEvolution);
	DECLARE_CYCLE_STAT(TEXT("MinEvolution::ApplyConstraintsPhase2"), STAT_MinEvolution_ApplyConstraintsPhase2, STATGROUP_ChaosMinEvolution);
	DECLARE_CYCLE_STAT(TEXT("MinEvolution::ApplyConstraintsPhase3"), STAT_MinEvolution_ApplyConstraintsPhase3, STATGROUP_ChaosMinEvolution);
	DECLARE_CYCLE_STAT(TEXT("MinEvolution::DetectCollisions"), STAT_MinEvolution_DetectCollisions, STATGROUP_ChaosMinEvolution);

	//
	//
	//

	// Forced iteration count to evaluate worst-case behaviour for a given simulation
	bool Chaos_MinEvolution_ForceMaxConstraintIterations = false;
	FAutoConsoleVariableRef CVarChaosMinEvolutionForceMaxConstraintIterations(TEXT("p.Chaos.MinEvolution.ForceMaxConstraintIterations"), Chaos_MinEvolution_ForceMaxConstraintIterations, TEXT("Whether to force constraints to always use the worst-case maximum number of iterations"));

	//
	//
	//

	FPBDMinEvolution::FPBDMinEvolution(FRigidParticleSOAs& InParticles, TArrayCollectionArray<FVec3>& InPrevX, TArrayCollectionArray<FRotation3>& InPrevR, FCollisionDetector& InCollisionDetector)
		: Particles(InParticles)
		, CollisionDetector(InCollisionDetector)
		, ParticlePrevXs(InPrevX)
		, ParticlePrevRs(InPrevR)
		, ConstraintSolver(Private::FIterationSettings(0,0,0))
		, Gravity(FVec3(0))
		, SimulationSpaceSettings()
		, bRewindVelocities(false)
	{
	}

	FPBDMinEvolution::~FPBDMinEvolution()
	{
	}

	void FPBDMinEvolution::AddConstraintContainer(FPBDConstraintContainer& InContainer, const int32 Priority)
	{
		// Do not add twice
		check(InContainer.GetContainerId() == INDEX_NONE);

		const int32 ContainerId = ConstraintContainers.Add(&InContainer);
		InContainer.SetContainerId(ContainerId);

		ConstraintSolver.SetConstraintSolver(ContainerId, InContainer.CreateSceneSolver(Priority));
	}

	void FPBDMinEvolution::SetConstraintContainerPriority(const int32 ContainerId, const int32 Priority)
	{
		ConstraintSolver.SetConstraintSolverPriority(ContainerId, Priority);
	}

	void FPBDMinEvolution::Advance(const FReal StepDt, const int32 NumSteps, const FReal RewindDt)
	{
		SCOPE_CYCLE_COUNTER(STAT_MinEvolution_Advance);

		PrepareTick();

		if (RewindDt > UE_SMALL_NUMBER)
		{
			Rewind(StepDt, RewindDt);
		}

		for (int32 Step = 0; Step < NumSteps; ++Step)
		{
			// StepFraction: how much of the remaining time this step represents, used to interpolate kinematic targets
			// E.g., for 4 steps this will be: 1/4, 1/2, 3/4, 1
			const FReal StepFraction = (FReal)(Step + 1) / (FReal)(NumSteps);

			UE_LOGF(LogChaosMinEvolution, Verbose, "Advance dt = %f [%d/%d]", StepDt, Step + 1, NumSteps);

			AdvanceOneTimeStep(StepDt, StepFraction);
		}

		for (TTransientPBDRigidParticleHandle<FReal, 3>& Particle : Particles.GetActiveParticlesView())
		{
			if (Particle.ObjectState() == EObjectStateType::Dynamic)
			{
				Particle.Acceleration() = FVec3(0);
				Particle.AngularAcceleration() = FVec3(0);
			}
		}

		UnprepareTick();
	}

	void FPBDMinEvolution::AdvanceOneTimeStep(const FReal Dt, const FReal StepFraction)
	{
		SCOPE_CYCLE_COUNTER(STAT_MinEvolution_AdvanceOneTimeStep);

		CVD_SCOPE_TRACE_SOLVER_STEP(CVDDC_Default, TEXT("Evolution Advance"));

		Integrate(Dt);

		ApplyKinematicTargets(Dt, StepFraction);

		DetectCollisions(Dt);

		if (Dt > 0)
		{
			GatherInput(Dt);

			ApplyConstraintsPhase1(Dt);

			ApplyConstraintsPhase2(Dt);

			ApplyConstraintsPhase3(Dt);

			ScatterOutput(Dt);
		}

		CVD_TRACE_CONSTRAINTS_CONTAINER(ConstraintContainers);

		CVD_TRACE_PARTICLES(Particles.GetParticleHandles());
	}

	// A opportunity for systems to allocate buffers for the duration of the tick, if they have enough info to do so
	void FPBDMinEvolution::PrepareTick()
	{
		SCOPE_CYCLE_COUNTER(STAT_MinEvolution_PrepareTick);

		for (FPBDConstraintContainer* ConstraintContainer : ConstraintContainers)
		{
			ConstraintContainer->PrepareTick();
		}
	}

	void FPBDMinEvolution::UnprepareTick()
	{
		SCOPE_CYCLE_COUNTER(STAT_MinEvolution_UnprepareTick);

		for (FPBDConstraintContainer* ConstraintContainer : ConstraintContainers)
		{
			ConstraintContainer->UnprepareTick();
		}
	}

	// Update X/R as if we started the next tick 'RewindDt' seconds ago.
	void FPBDMinEvolution::Rewind(FReal Dt, FReal RewindDt)
	{
		SCOPE_CYCLE_COUNTER(STAT_MinEvolution_Rewind);

		const FReal T = (Dt - RewindDt) / Dt;
		UE_LOGF(LogChaosMinEvolution, Verbose, "Rewind dt = %f; rt = %f; T = %f", Dt, RewindDt, T);
		for (TTransientPBDRigidParticleHandle<FReal, 3>& Particle : Particles.GetActiveParticlesView())
		{
			if (Particle.ObjectState() == EObjectStateType::Dynamic)
			{
				Particle.SetX(FVec3::Lerp(Particle.Handle()->AuxilaryValue(ParticlePrevXs), Particle.GetX(), T));
				Particle.SetRf(FRotation3f::Slerp(FRotation3f(Particle.Handle()->AuxilaryValue(ParticlePrevRs)), Particle.GetRf(), FRealSingle(T)));

				if (bRewindVelocities)
				{
					Particle.SetVf(FVec3f::Lerp(Particle.GetPreVf(), Particle.GetVf(), FRealSingle(T)));
					Particle.SetWf(FVec3f::Lerp(Particle.GetPreWf(), Particle.GetWf(), FRealSingle(T)));
				}
			}
			else if (Particle.ObjectState() == EObjectStateType::Kinematic)
			{
				// We can't rewind velocities for kinematics, and anyway they will get recalculated
				// anyway when the kinematic targets are applied.
				Particle.SetX(Particle.GetX() - Particle.GetV() * RewindDt);
				Particle.SetRf(FRotation3f::IntegrateRotationWithAngularVelocity(Particle.GetRf(), -Particle.GetWf(), FRealSingle(RewindDt)));
			}
		}

		for (auto& Particle : Particles.GetActiveKinematicParticlesView())
		{
			Particle.SetX(Particle.GetX() - Particle.GetV() * RewindDt);
			Particle.SetRf(FRotation3f::IntegrateRotationWithAngularVelocity(Particle.GetRf(), -Particle.GetWf(), FRealSingle(RewindDt)));
		}
	}

	void FPBDMinEvolution::Integrate(FReal Dt)
	{
		SCOPE_CYCLE_COUNTER(STAT_MinEvolution_Integrate);
		const FVec3 BoundsExpansion = FVec3(CollisionDetector.GetCollisionContainer().GetDetectorSettings().BoundsExpansion);

		for (TTransientPBDRigidParticleHandle<FReal, 3>& Particle : Particles.GetActiveParticlesView())
		{
			if (Particle.ObjectState() == EObjectStateType::Dynamic)
			{
				Particle.SetPreVf(Particle.GetVf());
				Particle.SetPreWf(Particle.GetWf());

				const FVec3 XCoM = Particle.XCom();
				const FRotation3 RCoM = Particle.RCom();

				// Forces and torques
				const FMatrix33 WorldInvI = Utilities::ComputeWorldSpaceInertia(RCoM, Particle.InvI());
				FVec3 DV = Particle.Acceleration() * Dt + Particle.LinearImpulseVelocity();
				FVec3 DW = Particle.AngularAcceleration() * Dt + Particle.AngularImpulseVelocity();
				FVec3 TargetV = FVec3(0);
				FVec3 TargetW = FVec3(0);

				// Gravity
				if (Particle.GravityEnabled())
				{
					DV += Gravity * Dt;
				}

				FVec3 V = Particle.GetV();
				FVec3 W = Particle.GetW();

				// Moving and accelerating simulation frame
				// https://en.wikipedia.org/wiki/Rotating_reference_frame
				if (SimulationSpaceSettings.bEnabled)
				{
					const FVec3 WxR = FVec3::CrossProduct(SimulationSpace.AngularVelocity, XCoM);
					const FVec3 WxWxR = FVec3::CrossProduct(SimulationSpace.AngularVelocity, WxR);
					const FVec3 WxV = FVec3::CrossProduct(SimulationSpace.AngularVelocity, V);
					const FVec3 AxR = FVec3::CrossProduct(SimulationSpace.AngularAcceleration, XCoM);

					// Calculate the phantom forces
					const FVec3 CoriolisAcc = SimulationSpaceSettings.CoriolisAlpha * 2.0f * WxV;
					const FVec3 CentrifugalAcc = SimulationSpaceSettings.CentrifugalAlpha * WxWxR;
					const FVec3 EulerAcc = SimulationSpaceSettings.EulerAlpha * AxR;
					const FVec3 LinearAcc = SimulationSpaceSettings.LinearAccelerationAlpha * SimulationSpace.LinearAcceleration;
					const FVec3 AngularAcc = SimulationSpaceSettings.AngularAccelerationAlpha * SimulationSpace.AngularAcceleration;
					const FVec3 LinearDragAcc = SimulationSpaceSettings.ExternalLinearEtherDrag * (V + SimulationSpace.LinearVelocity + WxR);

					// Apply the phantom forces
					DV -= (LinearAcc + LinearDragAcc + CoriolisAcc + CentrifugalAcc + EulerAcc) * Dt;
					DW -= AngularAcc * Dt;

					// Pass on some part of world-space motion to the local-space drag calculation below.
					// E.g., for LinearVelocityAlpha=1.0, and if LinearDrag is maxed out below, then the particle's local space velocity 
					// would end up being exactly negative SimulationSpace.LinearVelocity. If LinearVelocityAlpha=0.0, then the 
					// simulation-space velocity does not affect local drag.
					TargetV = -SimulationSpaceSettings.LinearVelocityAlpha * SimulationSpace.LinearVelocity;
					TargetW = -SimulationSpaceSettings.AngularVelocityAlpha * SimulationSpace.AngularVelocity;
				}

				// New velocity
				const FReal LinearDamping = FMath::Min(FReal(1), Particle.LinearEtherDrag() * Dt);
				const FReal AngularDamping = FMath::Min(FReal(1), Particle.AngularEtherDrag() * Dt);
				V = FMath::Lerp(V + DV, TargetV, LinearDamping);
				W = FMath::Lerp(W + DW, TargetW, AngularDamping);

				if (Particle.GyroscopicTorqueEnabled())
				{
					W = Utilities::GetAngularVelocityAdjustedForGyroscopicTorques(RCoM, Particle.I(), W, Dt);
				}

				// New position
				const FVec3 PCoM = XCoM + V * Dt;
				const FRotation3 QCoM = FRotation3::IntegrateRotationWithAngularVelocity(RCoM, W, Dt);

				// Update particle state (forces are not zeroed until the end of the frame)
				Particle.SetTransformPQCom(PCoM, QCoM);
				Particle.SetV(V);
				Particle.SetW(W);
				Particle.LinearImpulseVelocity() = FVec3(0);
				Particle.AngularImpulseVelocity() = FVec3(0);

				// Update cached world space state, including bounds. We use the Swept bounds update so that the bounds includes P,Q and X,Q.
				// This is because when we have joints, they often pull bodies back to their original positions, so we need to know if there
				// are contacts at that location.
				Particle.UpdateWorldSpaceStateSwept(FRigidTransform3(Particle.GetP(), Particle.GetQ()), BoundsExpansion, -V * Dt);
			}
		}
	}

	// @todo(ccaulfield): dedupe (PBDRigidsEvolutionGBF)
	void FPBDMinEvolution::ApplyKinematicTargets(FReal Dt, FReal StepFraction)
	{
		SCOPE_CYCLE_COUNTER(STAT_MinEvolution_KinematicTargets);

		check(StepFraction > (FReal)0);
		check(StepFraction <= (FReal)1);

		const bool IsLastStep = (FMath::IsNearlyEqual(StepFraction, (FReal)1, (FReal)UE_KINDA_SMALL_NUMBER));

		const auto& ApplyDynamicParticleKinematicTarget = 
		[Dt, StepFraction, IsLastStep]
		(FTransientPBDRigidParticleHandle& Particle, const int32 ParticleIndex)
		-> void
		{
			if (!Particle.IsKinematic())
			{
				return;
			}

			FKinematicTarget& KinematicTarget = Particle.KinematicTarget();
			const FVec3 CurrentX = Particle.GetX();
			const FRotation3f CurrentR = Particle.GetR();
			constexpr FReal MinDt = 1e-6f;

			bool bMoved = false;
			switch (KinematicTarget.GetMode())
			{
			case EKinematicTargetMode::None:
				// Nothing to do
				break;

			case EKinematicTargetMode::Reset:
			{
				// Reset velocity and then switch to do-nothing mode
				Particle.SetVf(FVec3f(0, 0, 0));
				Particle.SetWf(FVec3f(0, 0, 0));
				KinematicTarget.SetMode(EKinematicTargetMode::None);
				break;
			}

			case EKinematicTargetMode::Position:
			{
				// Move to kinematic target and update velocities to match
				// Target positions only need to be processed once, and we reset the velocity next frame (if no new target is set)
				FVec3 NewX;
				FRotation3f NewR;
				if (IsLastStep)
				{
					NewX = KinematicTarget.GetPosition();
					NewR = KinematicTarget.GetRotation();
					KinematicTarget.SetMode(EKinematicTargetMode::Reset);
				}
				else
				{
					// as a reminder, stepfraction is the remaing fraction of the step from the remaining steps
					// for total of 4 steps and current step of 2, this will be 1/3 ( 1 step passed, 3 steps remains )
					NewX = FVec3::Lerp(CurrentX, KinematicTarget.GetPosition(), StepFraction);
					NewR = FRotation3f::Slerp(CurrentR, KinematicTarget.GetRotation(), decltype(FQuat4f::X)(StepFraction));
				}

				const bool bPositionChanged = !FVec3::IsNearlyEqual(NewX, CurrentX, UE_SMALL_NUMBER);
				const bool bRotationChanged = !FRotation3f::IsNearlyEqual(NewR, CurrentR, UE_KINDA_SMALL_NUMBER);
				bMoved = bPositionChanged || bRotationChanged;
				FVec3 NewV = FVec3(0);
				FVec3 NewW = FVec3(0);
				if (Dt > MinDt)
				{
					if (bPositionChanged)
					{
						NewV = FVec3::CalculateVelocity(CurrentX, NewX, Dt);
					}
					if (bRotationChanged)
					{
						NewW = FRotation3::CalculateAngularVelocity(CurrentR, NewR, Dt);
					}
				}
				Particle.SetX(NewX);
				Particle.SetRf(NewR);
				Particle.SetV(NewV);
				Particle.SetW(NewW);

				break;
			}

			case EKinematicTargetMode::Velocity:
			{
				// Move based on velocity
				bMoved = true;
				Particle.SetX(Particle.GetX() + Particle.GetV() * Dt);
				Particle.SetRf(FRotation3f::IntegrateRotationWithAngularVelocity(Particle.GetRf(), Particle.GetWf(), static_cast<FRealSingle>(Dt)));
				break;
			}
			}
			
			// Set positions and previous velocities if we can
			// Note: At present kinematics are in fact rigid bodies
			Particle.SetP(Particle.GetX());
			Particle.SetQf(Particle.GetRf());
			Particle.SetPreV(Particle.GetV());
			Particle.SetPreWf(Particle.GetWf());

			if (bMoved)
			{
				if (!Particle.CCDEnabled())
				{
					Particle.UpdateWorldSpaceState(FRigidTransform3(Particle.GetP(), Particle.GetQ()), FVec3(0));
				}
				else
				{
					Particle.UpdateWorldSpaceStateSwept(FRigidTransform3(Particle.GetP(), Particle.GetQ()), FVec3(0), -Particle.GetV() * Dt);
				}
			}
		};

		const auto& ApplyKinematicParticleKinematicTarget =
		[Dt, StepFraction, IsLastStep]
		(FTransientKinematicGeometryParticleHandle& Particle, const int32 ParticleIndex)
		-> void
		{
			FKinematicTarget& KinematicTarget = Particle.KinematicTarget();
			const FVec3 CurrentX = Particle.GetX();
			const FRotation3f CurrentR = Particle.GetR();
			constexpr FReal MinDt = 1e-6f;

			bool bMoved = false;
			switch (KinematicTarget.GetMode())
			{
			case EKinematicTargetMode::None:
				// Nothing to do
				break;

			case EKinematicTargetMode::Reset:
			{
				// Reset velocity and then switch to do-nothing mode
				Particle.SetVf(FVec3f(0, 0, 0));
				Particle.SetWf(FVec3f(0, 0, 0));
				KinematicTarget.SetMode(EKinematicTargetMode::None);
				break;
			}

			case EKinematicTargetMode::Position:
			{
				// Move to kinematic target and update velocities to match
				// Target positions only need to be processed once, and we reset the velocity next frame (if no new target is set)
				FVec3 NewX;
				FRotation3f NewR;
				if (IsLastStep)
				{
					NewX = KinematicTarget.GetPosition();
					NewR = KinematicTarget.GetRotation();
					KinematicTarget.SetMode(EKinematicTargetMode::Reset);
				}
				else
				{
					// as a reminder, stepfraction is the remaing fraction of the step from the remaining steps
					// for total of 4 steps and current step of 2, this will be 1/3 ( 1 step passed, 3 steps remains )
					NewX = FVec3::Lerp(CurrentX, KinematicTarget.GetPosition(), StepFraction);
					NewR = FRotation3f::Slerp(CurrentR, KinematicTarget.GetRotation(), decltype(FQuat4f::X)(StepFraction));
				}

				const bool bPositionChanged = !FVec3::IsNearlyEqual(NewX, CurrentX, UE_SMALL_NUMBER);
				const bool bRotationChanged = !FRotation3::IsNearlyEqual(NewR, CurrentR, UE_SMALL_NUMBER);
				bMoved = bPositionChanged || bRotationChanged;
				FVec3 NewV = FVec3(0);
				FVec3f NewW = FVec3(0);
				if (Dt > MinDt)
				{
					if (bPositionChanged)
					{
						NewV = FVec3::CalculateVelocity(CurrentX, NewX, Dt);
					}
					if (bRotationChanged)
					{
						NewW = FRotation3f::CalculateAngularVelocity(CurrentR, NewR, static_cast<FRealSingle>(Dt));
					}
				}
				Particle.SetX(NewX);
				Particle.SetRf(NewR);
				Particle.SetV(NewV);
				Particle.SetWf(NewW);

				break;
			}

			case EKinematicTargetMode::Velocity:
			{
				// Move based on velocity
				bMoved = true;
				Particle.SetX(Particle.GetX() + Particle.GetV() * Dt);
				Particle.SetRf(FRotation3f::IntegrateRotationWithAngularVelocity(Particle.GetRf(), Particle.GetWf(), FRealSingle(Dt)));
				break;
			}
			}

			if (bMoved)
			{
				Particle.UpdateWorldSpaceState(FRigidTransform3(Particle.GetX(), Particle.GetR()), FVec3(0));
			}
		};

		// Apply kinematic targets
		// 
		// We could run the updates in parallel, but in practice we're more likely to get parallelism benefits
		// from running multiple characters in parallel, especially as often we'll only have a small number of 
		// objects to process here.
		bool bForceSingleThreaded = true;
		// All the real kinematic particles
		Particles.GetActiveKinematicParticlesView().ParallelFor(ApplyKinematicParticleKinematicTarget, bForceSingleThreaded);
		// All the dynamic particles plus kinematic ones.
		Particles.GetActiveDynamicMovingKinematicParticlesView().ParallelFor(ApplyDynamicParticleKinematicTarget, bForceSingleThreaded);

		// done with update, let's clear the tracking structures
		if (IsLastStep)
		{
			Particles.UpdateAllMovingKinematic();
		}

		// If we changed any particle state, the views need to be refreshed
		Particles.UpdateDirtyViews();
	}

	void FPBDMinEvolution::DetectCollisions(FReal Dt)
	{
		SCOPE_CYCLE_COUNTER(STAT_MinEvolution_DetectCollisions);

		for (FPBDConstraintContainer* ConstraintContainer : ConstraintContainers)
		{
			ConstraintContainer->UpdatePositionBasedState(Dt);
		}

		CollisionDetector.DetectCollisions(Dt, nullptr);
		CollisionDetector.GetCollisionContainer().GetConstraintAllocator().PruneExpiredItems();
		CollisionDetector.GetCollisionContainer().GetConstraintAllocator().SortConstraintsHandles();
	}

	void FPBDMinEvolution::GatherInput(FReal Dt)
	{
		SCOPE_CYCLE_COUNTER(STAT_MinEvolution_Gather);

		ConstraintSolver.Reset();
		ConstraintSolver.AddConstraintsAndBodies();
		ConstraintSolver.GatherBodies(Dt);
		ConstraintSolver.GatherConstraints(Dt);
	}

	void FPBDMinEvolution::ScatterOutput(FReal Dt)
	{
		SCOPE_CYCLE_COUNTER(STAT_MinEvolution_Scatter);

		ConstraintSolver.ScatterConstraints(Dt);
		ConstraintSolver.ScatterBodies(Dt);

		for (auto& Particle : Particles.GetActiveParticlesView())
		{
			Particle.Handle()->AuxilaryValue(ParticlePrevXs) = Particle.GetX();
			Particle.Handle()->AuxilaryValue(ParticlePrevRs) = Particle.GetR();
			Particle.SetX(Particle.GetP());
			Particle.SetR(Particle.GetQ());
		}
	}

	void FPBDMinEvolution::ApplyConstraintsPhase1(FReal Dt)
	{
		SCOPE_CYCLE_COUNTER(STAT_MinEvolution_ApplyConstraintsPhase1);

		ConstraintSolver.PreApplyPositionConstraints(Dt);
		ConstraintSolver.ApplyPositionConstraints(Dt);
	}

	void FPBDMinEvolution::ApplyConstraintsPhase2(FReal Dt)
	{
		SCOPE_CYCLE_COUNTER(STAT_MinEvolution_ApplyConstraintsPhase2);

		ConstraintSolver.PreApplyVelocityConstraints(Dt);
		ConstraintSolver.ApplyVelocityConstraints(Dt);
	}

	void FPBDMinEvolution::ApplyConstraintsPhase3(FReal Dt)
	{
		SCOPE_CYCLE_COUNTER(STAT_MinEvolution_ApplyConstraintsPhase3);

		ConstraintSolver.PreApplyProjectionConstraints(Dt);
		ConstraintSolver.ApplyProjectionConstraints(Dt);
	}
}
