// Copyright Epic Games, Inc. All Rights Reserved.

#define EXPLICIT_VECTOR4 1

#include "Math/Vector.isph"
#include "Math/Quat.isph"
#include "Math/Transform.isph"
#include "Chaos/Matrix33.isph"

export uniform int32 SizeofFTransform()
{
	return sizeof(uniform FTransform);
}


struct FAABB
{
	FVector Min;
	FVector Max;
};

export uniform int32 SizeofFAABB()
{
	return sizeof(uniform FAABB);
}


struct FPBDRigidArrays
{
	int NumParticles;
	int8* ObjectState;
	FVector* X;
	FVector* P;
	FVector4* R;
	FVector4* Q;
	FVector* V;
	FVector* PreV;
	FVector* W;
	FVector* PreW;
	FVector* CenterOfMass;
	FVector4* RotationOfMass;
	FReal* InvM;
	FMatrix33* InvI;
	FVector* F;
	FVector* T;
	FVector* LinearImpulse;
	FVector* AngularImpulse;
	int8* Disabled;
	int8* GravityEnabled;
	FReal* LinearEtherDrag;
	FReal* AngularEtherDrag;
	int8* HasBounds;
	FAABB* LocalBounds;
	FAABB* WorldBounds;
};

export uniform int32 SizeofFPBDRigidArrays()
{
	return sizeof(uniform FPBDRigidArrays);
}


struct FSimulationSpace
{
	FTransform Transform;
	FVector LinearAcceleration;
	FVector AngularAcceleration;
	FVector LinearVelocity;
	FVector AngularVelocity;
};

export uniform int32 SizeofFSimulationSpace()
{
	return sizeof(uniform FSimulationSpace);
}


struct FSimulationSpaceSettings
{
	// Global multipler on the effects of simulation space movement
	FReal MasterAlpha;

	// How much of the simulation frame's linear acceleration to pass onto the particles
	FReal LinearAccelerationAlpha;

	// How much of the coriolis force to apply to the particles
	FReal CoriolisAlpha;

	// How much of the centrifugal force to apply to the particles
	FReal CentrifugalAlpha;

	// How much of the euler force to apply to the particles
	FReal EulerAlpha;

	// How much of the simulation frame's angular acceleration to pass onto the particles
	FReal AngularAccelerationAlpha;

	// How much of the simulation frame's linear velocity to pass onto the particles (linear ether drag)
	FReal LinearVelocityAlpha;

	// How much of the simulation frame's angular velocity to pass onto the particles (angular ether drag)
	FReal AngularVelocityAlpha;

	// An additional linear drag (on top of the EtherDrag that can be set on the physics asset)
	FVector ExternalLinearEtherDrag;
};

export uniform int32 SizeofFSimulationSpaceSettings()
{
	return sizeof(uniform FSimulationSpaceSettings);
}

static inline uniform FMatrix33 ComputeWorldSpaceInertia(const uniform FVector4 &Q, const uniform FVector &I)
{
	const uniform FMatrix33 QM = MakeQuatRotationTranslationMatrix(Q);
	const uniform FMatrix33 L = SetMatrix33(I.V[0], I.V[1], I.V[2]);
	return MultiplyAB(QM, MultiplyABt(L, QM));
}

static inline uniform FMatrix33 ComputeWorldSpaceInertia(const uniform FVector4 &Q, const uniform FMatrix33 &I)
{
	const uniform FMatrix33 QM = MakeQuatRotationTranslationMatrix(Q);
	return MultiplyAB(QM, MultiplyABt(I, QM));
}

static inline uniform FVector4 IntegrateRotationWithAngularVelocity(const uniform FVector4& Q0, const uniform FVector& W, const uniform FReal Dt)
{
	uniform FVector4 R1 = Q0 + VectorQuaternionMultiply2(SetVector4(W.V[0], W.V[1], W.V[2], ZERO), Q0) * (Dt * ONE_HALF);
	return VectorNormalizeSafe(R1, VectorZero);
}

static inline uniform int8 EObjectStateType_Dynamic()
{
	// See EObjectStateType in Particles.h
	return 4;
}

export uniform int ValueOfEObjectStateTypeDynamic()
{
	return EObjectStateType_Dynamic();
}

static inline void GrowToInclude(uniform FVector &Min, uniform FVector &Max, const uniform FVector& V)
{
	Min = VectorMin(Min, V);
	Max = VectorMax(Max, V);
}

static inline void TransformedAABB(const uniform FTransform &SpaceTransform, const uniform FVector &Min, const uniform FVector &Max, uniform FVector &NewMin, uniform FVector &NewMax)
{
	const uniform FVector CurrentExtents = Max - Min;

	uniform FVector MinToNewSpace = TransformPosition(SpaceTransform, Min);
	uniform FVector MaxToNewSpace = MinToNewSpace;
	GrowToInclude(MinToNewSpace, MaxToNewSpace, TransformPosition(SpaceTransform, Max));

	GrowToInclude(MinToNewSpace, MaxToNewSpace, TransformPosition(SpaceTransform, Min + ForwardVector * CurrentExtents));
	GrowToInclude(MinToNewSpace, MaxToNewSpace, TransformPosition(SpaceTransform, Max - ForwardVector * CurrentExtents));
	GrowToInclude(MinToNewSpace, MaxToNewSpace, TransformPosition(SpaceTransform, Min + RightVector * CurrentExtents));
	GrowToInclude(MinToNewSpace, MaxToNewSpace, TransformPosition(SpaceTransform, Max - RightVector * CurrentExtents));
	GrowToInclude(MinToNewSpace, MaxToNewSpace, TransformPosition(SpaceTransform, Min + UpVector * CurrentExtents));
	GrowToInclude(MinToNewSpace, MaxToNewSpace, TransformPosition(SpaceTransform, Max - UpVector * CurrentExtents));

	NewMin = MinToNewSpace;
	NewMax = MaxToNewSpace;
}

static inline void ThickenSymmetrically(uniform FVector &Min, uniform FVector &Max, const uniform FVector& V)
{
	Min = Min - V;
	Max = Max + V;
}

static inline void GrowByVector(uniform FVector &Min, uniform FVector &Max, const uniform FVector& V)
{
	Min = Min + VectorMin(V, ZeroVector);
	Max = Max + VectorMax(V, ZeroVector);
}

static inline uniform FVector InverseTransformVectorNoScale(const uniform FTransform& Transform, const uniform FVector &V)
{
	return VectorQuaternionRotateVector(QuatInverse(Transform.Rotation), V);
}

export void MinEvolutionIntegrate(
	uniform FReal Dt, 
	uniform FPBDRigidArrays& Rigids, 
	uniform FSimulationSpace& SimulationSpace, 
	uniform FSimulationSpaceSettings& SimulationSpaceSettings, 
	const uniform FVector& Gravity, 
	const uniform FReal BoundsExtension,
	const uniform FReal CullDistance)
{
	// Simulation space velocity and acceleration
	uniform FVector SpaceV = ZeroVector;	// Velocity
	uniform FVector SpaceW = ZeroVector;	// Angular Velocity
	uniform FVector SpaceA = ZeroVector;	// Acceleration
	uniform FVector SpaceB = ZeroVector;	// Angular Acceleration
	if (SimulationSpaceSettings.MasterAlpha > ZERO)
	{
		SpaceV = InverseTransformVectorNoScale(SimulationSpace.Transform, SimulationSpace.LinearVelocity);
		SpaceW = InverseTransformVectorNoScale(SimulationSpace.Transform, SimulationSpace.AngularVelocity);
		SpaceA = InverseTransformVectorNoScale(SimulationSpace.Transform, SimulationSpace.LinearAcceleration);
		SpaceB = InverseTransformVectorNoScale(SimulationSpace.Transform, SimulationSpace.AngularAcceleration);
	}

	for (uniform int ParticleIndex = 0; ParticleIndex < Rigids.NumParticles; ++ParticleIndex)
	{
		const uniform bool bDisabled = Rigids.Disabled[ParticleIndex];
		const uniform bool bDynamic = (Rigids.ObjectState[ParticleIndex] == EObjectStateType_Dynamic());
		const uniform bool bEnabled = !bDisabled && bDynamic;
		if (bEnabled)
		{
			Rigids.PreV[ParticleIndex] = Rigids.V[ParticleIndex];
			Rigids.PreW[ParticleIndex] = Rigids.W[ParticleIndex];

			const uniform FVector XCoM = Rigids.X[ParticleIndex] + VectorQuaternionRotateVector(Rigids.R[ParticleIndex], Rigids.CenterOfMass[ParticleIndex]);
			const uniform FVector4 RCoM = VectorQuaternionMultiply2(Rigids.R[ParticleIndex], Rigids.RotationOfMass[ParticleIndex]);

			// Forces and torques
			const uniform FMatrix33 WorldInvI = ComputeWorldSpaceInertia(RCoM, Rigids.InvI[ParticleIndex]);
			uniform FVector DV = Rigids.InvM[ParticleIndex] * (Rigids.F[ParticleIndex] * Dt + Rigids.LinearImpulse[ParticleIndex]);
			uniform FVector DW = Multiply(WorldInvI, (Rigids.T[ParticleIndex] * Dt + Rigids.AngularImpulse[ParticleIndex]));
			uniform FVector TargetV = ZeroVector;
			uniform FVector TargetW = ZeroVector;

			// Gravity
			const uniform bool bGravityEnabled = Rigids.GravityEnabled[ParticleIndex];
			if (bGravityEnabled)
			{
				DV = DV + Gravity * Dt;
			}

			// Moving and accelerating simulation frame
			// https://en.wikipedia.org/wiki/Rotating_reference_frame
			if (SimulationSpaceSettings.MasterAlpha > ZERO)
			{
				const uniform FVector CoriolisAcc = SimulationSpaceSettings.CoriolisAlpha * TWO * VectorCross(SpaceW, Rigids.V[ParticleIndex]);
				const uniform FVector CentrifugalAcc = SimulationSpaceSettings.CentrifugalAlpha * VectorCross(SpaceW, VectorCross(SpaceW, XCoM));
				const uniform FVector EulerAcc = SimulationSpaceSettings.EulerAlpha * VectorCross(SpaceB, XCoM);
				const uniform FVector LinearAcc = SimulationSpaceSettings.LinearAccelerationAlpha * SpaceA;
				const uniform FVector AngularAcc = SimulationSpaceSettings.AngularAccelerationAlpha * SpaceB;
				const uniform FVector LinearDragAcc = SimulationSpaceSettings.ExternalLinearEtherDrag * SpaceV;
				DV = DV - SimulationSpaceSettings.MasterAlpha * (LinearAcc + LinearDragAcc + CoriolisAcc + CentrifugalAcc + EulerAcc) * Dt;
				DW = DW - SimulationSpaceSettings.MasterAlpha * AngularAcc * Dt;
				TargetV = -SimulationSpaceSettings.MasterAlpha * SimulationSpaceSettings.LinearVelocityAlpha * SpaceV;
				TargetW = -SimulationSpaceSettings.MasterAlpha * SimulationSpaceSettings.AngularVelocityAlpha * SpaceW;
			}

			// New velocity
			const uniform FReal LinearDrag = min(ONE, Rigids.LinearEtherDrag[ParticleIndex] * Dt);
			const uniform FReal AngularDrag = min(ONE, Rigids.AngularEtherDrag[ParticleIndex] * Dt);
			const uniform FVector VCoM = VectorLerp(Rigids.V[ParticleIndex] + DV, TargetV, LinearDrag);
			const uniform FVector WCoM = VectorLerp(Rigids.W[ParticleIndex] + DW, TargetW, AngularDrag);

			// New position
			const uniform FVector PCoM = XCoM + VCoM * Dt;
			const uniform FVector4 QCoM = IntegrateRotationWithAngularVelocity(RCoM, WCoM, Dt);

			// Update particle state (forces are not zeroed until the end of the frame)
			const uniform FVector4 QActor = VectorQuaternionMultiply2(QCoM, QuatInverse(Rigids.RotationOfMass[ParticleIndex]));
			const uniform FVector PActor = PCoM - VectorQuaternionRotateVector(QActor, Rigids.CenterOfMass[ParticleIndex]);
			Rigids.P[ParticleIndex] = PActor;
			Rigids.Q[ParticleIndex] = QActor;

			Rigids.V[ParticleIndex] = VCoM;
			Rigids.W[ParticleIndex] = WCoM;
			Rigids.LinearImpulse[ParticleIndex] = ZeroVector;
			Rigids.AngularImpulse[ParticleIndex] = ZeroVector;

			// Update world-space bounds
			if (Rigids.HasBounds[ParticleIndex])
			{
				uniform FTransform SpaceTransform;
				SpaceTransform.Rotation = Rigids.Q[ParticleIndex];
				SpaceTransform.Translation = SetVector4(Rigids.P[ParticleIndex], ONE);
				SpaceTransform.Scale3D = SetVector4(OneVector, ONE);
			
				uniform FVector WorldSpaceBoundsMin;
				uniform FVector WorldSpaceBoundsMax;
				TransformedAABB(SpaceTransform, Rigids.LocalBounds[ParticleIndex].Min, Rigids.LocalBounds[ParticleIndex].Max, WorldSpaceBoundsMin, WorldSpaceBoundsMax);
			
				ThickenSymmetrically(WorldSpaceBoundsMin, WorldSpaceBoundsMax, BoundsExtension * (WorldSpaceBoundsMax - WorldSpaceBoundsMin));
			
				// Dynamic bodies may get pulled back into their old positions by joints - make sure we find collisions that may prevent this
				// We could add the AABB at X/R here, but I'm avoiding another call to TransformedAABB. Hopefully this is good enough.
				GrowByVector(WorldSpaceBoundsMin, WorldSpaceBoundsMax, Rigids.X[ParticleIndex] - Rigids.P[ParticleIndex]);
			
				ThickenSymmetrically(WorldSpaceBoundsMin, WorldSpaceBoundsMax, SetVector(CullDistance, CullDistance, CullDistance));
			
				Rigids.WorldBounds[ParticleIndex].Min = WorldSpaceBoundsMin;
				Rigids.WorldBounds[ParticleIndex].Max = WorldSpaceBoundsMax;
			}
		}
	}
}

