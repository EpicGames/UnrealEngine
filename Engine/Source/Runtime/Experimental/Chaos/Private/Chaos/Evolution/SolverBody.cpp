// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/Evolution/SolverBody.h"
#include "Chaos/Particle/ParticleUtilities.h"

//PRAGMA_DISABLE_OPTIMIZATION

namespace Chaos
{

	void FSolverBody::UpdateRotationDependentState()
	{
		if (IsDynamic())
		{
			State.InvI = Utilities::ComputeWorldSpaceInertia(State.Q, State.InvILocal);
		}
	}

	void SolverQuaternionNormalizeApprox(FRotation3& InOutQ)
	{
		//constexpr FReal Tolerance = FReal(2.107342e-08);	// For full double-precision accuracy
		constexpr FReal ToleranceF = FReal(0.001);

#if PLATFORM_ENABLE_VECTORINTRINSICS
		using QuatVectorRegister = FRotation3::QuatVectorRegister;

		constexpr VectorRegister Tolerance = MakeVectorRegisterConstant(ToleranceF, ToleranceF, ToleranceF, ToleranceF);
		constexpr VectorRegister One = MakeVectorRegisterConstant(FReal(1), FReal(1), FReal(1), FReal(1));
		constexpr VectorRegister Two = MakeVectorRegisterConstant(FReal(2), FReal(2), FReal(2), FReal(2));

		//const FReal QSq = SizeSquared();
		const QuatVectorRegister Q = VectorLoadAligned(&InOutQ);
		const QuatVectorRegister QSq = VectorDot4(Q, Q);

		// if (FMath::Abs(FReal(1) - QSq) < Tolerance)
		const QuatVectorRegister ToleranceCheck = VectorAbs(VectorSubtract(One, QSq));
		if (VectorMaskBits(VectorCompareLE(ToleranceCheck, Tolerance)) != 0)
		{
			// Q * (FReal(2) / (FReal(1) + QSq))
			const QuatVectorRegister Denom = VectorAdd(One, QSq);
			const QuatVectorRegister Mult = VectorDivide(Two, Denom);
			const QuatVectorRegister Result = VectorMultiply(Q, Mult);
			VectorStoreAligned(Result, &InOutQ);
		}
		else
		{
			// Q / QLen
			// @todo(chaos): with doubles VectorReciprocalSqrt does twice as many sqrts as we need and also has a divide
			const QuatVectorRegister Result = VectorNormalize(Q);
			VectorStoreAligned(Result, &InOutQ);
		}
#else
		const FReal QSq = InOutQ.SizeSquared();
		if (FMath::Abs(FReal(1) - QSq) < ToleranceF)
		{
			InOutQ *= (FReal(2) / (FReal(1) + QSq));
		}
		else
		{
			InOutQ.Normalize();
		}
#endif

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		ensure(!InOutQ.ContainsNaN());
		ensure(InOutQ.IsNormalized());
#endif
	}

	FRotation3 SolverQuaternionApplyAngularDeltaApprox(const FRotation3& InQ0, const FVec3& InDR)
	{
		FRotation3 Q1 = InQ0 + (FRotation3::FromElements(InDR, FReal(0)) * InQ0) * FReal(0.5);
		SolverQuaternionNormalizeApprox(Q1);
		return Q1;
	}
}

