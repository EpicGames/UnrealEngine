// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/Evolution/SolverBodyContainer.h"
#include "Chaos/Particle/ParticleUtilities.h"

namespace Chaos
{
	inline void UpdateSolverBodyFromParticle(FSolverBody& SolverBody, const FGeometryParticleHandle* Particle, const FReal Dt)
	{
		check(Particle != nullptr);

		SolverBody.Reset();
		SolverBody.SetLevel(0);

		const FPBDRigidParticleHandle* RigidParticle = Particle->CastToRigidParticle();
		const FKinematicGeometryParticleHandle* KinematicParticle = Particle->CastToKinematicParticle();

		if ((RigidParticle != nullptr) && RigidParticle->IsDynamic())
		{
			// Dynamic particle
			SolverBody.SetX(RigidParticle->XCom());
			SolverBody.SetR(RigidParticle->RCom());
			SolverBody.SetP(RigidParticle->PCom());
			SolverBody.SetQ(RigidParticle->QCom());
			SolverBody.SetCoM(RigidParticle->CenterOfMass());
			SolverBody.SetRoM(RigidParticle->RotationOfMass());
			if (!RigidParticle->IsSleeping()) // Awake
			{
				SolverBody.SetV(RigidParticle->GetV());
				SolverBody.SetW(RigidParticle->GetW());
				SolverBody.SetInvM(RigidParticle->InvM());
				SolverBody.SetInvILocal(RigidParticle->ConditionedInvI());
				// Note: SetInvILocal also calculates InvI for dynamic
			}
			else // Sleeping
			{
				// In the solver, we want sleeping particles to behave like kinematics
				// since they do not experience any positional impulse and position correction.
				// Thus, we set velocity, mass, and inertia to zero while using the position and orientation of the rigid particle.
				SolverBody.SetV(FVec3(0));
				SolverBody.SetW(FVec3(0));
				SolverBody.SetInvM(0);
				SolverBody.SetInvILocal(FVec3(0));	// Note: does not set InvI for Kinematic
				SolverBody.SetInvI(FMatrix33(0));
			}
		}
		else
		{
			// Static or kinematic particle
			SolverBody.SetP(Particle->GetX());
			SolverBody.SetQ(Particle->GetR());
			SolverBody.SetCoM(FVec3(0));
			SolverBody.SetRoM(FRotation3::FromIdentity());
			if (KinematicParticle != nullptr)
			{
				// Kinematic particle
				SolverBody.SetX(KinematicParticle->GetX() - KinematicParticle->GetV() * Dt);
				SolverBody.SetR(Particle->GetR());
				if (!KinematicParticle->GetW().IsNearlyZero())
				{
					SolverBody.SetR(FRotation3::IntegrateRotationWithAngularVelocity(KinematicParticle->GetR(), -KinematicParticle->GetW(), Dt));
				}
				SolverBody.SetV(KinematicParticle->GetV());
				SolverBody.SetW(KinematicParticle->GetW());
			}
			else
			{
				SolverBody.SetX(Particle->GetX());
				SolverBody.SetR(Particle->GetR());
				SolverBody.SetV(FVec3(0));
				SolverBody.SetW(FVec3(0));
			}
			SolverBody.SetInvM(0);
			SolverBody.SetInvILocal(FVec3(0));	// Note: does not set InvI for Kinematic
			SolverBody.SetInvI(FMatrix33(0));
		}
	}

	inline void UpdateParticleFromSolverBody(FGeometryParticleHandle* Particle, const FSolverBody& SolverBody)
	{
		check(Particle != nullptr);
		check(SolverBody.IsDynamic());

		// Set the particle state (only dynamics will have changed)
		if (FPBDRigidParticleHandle* RigidParticle = Particle->CastToRigidParticle())
		{
			RigidParticle->SetTransformPQCom(SolverBody.P(), SolverBody.Q());
			RigidParticle->SetV(SolverBody.V());
			RigidParticle->SetW(SolverBody.W());

			// Reset SolverBodyIndex cookie every step - it will be reassigned next step
			RigidParticle->SetSolverBodyIndex(INDEX_NONE);
		}
	}

	int32 FSolverBodyContainer::AddParticle(FGenericParticleHandle InParticle)
	{
		// No array resizing allowed (we want fixed pointers)
		check(Num() < Max());

		// NOTE: New solver body is completely uninitialized
		SolverBodies.Add();
		return Particles.Add(InParticle->Handle());
	}

	FSolverBody* FSolverBodyContainer::FindOrAdd(FGenericParticleHandle InParticle)
	{
		// For dynamic bodies, we store a cookie on the Particle that holds the solver body index
		// For kinematics we cannot do this because the kinematic may be in multiple islands and 
		// would require a different index for each island, so we use a local map instead. 
		int32 ItemIndex = InParticle->SolverBodyIndex();
	
		if (ItemIndex == INDEX_NONE)
		{
			// NOTE: IsDynamic() is true whether the particle is awake or asleep.
			// The SolverBodyIndex() for sleeping particles does not seem to be reset at each tick.
			// As a workaround, we use the same procedure as kinematic particles to make sure it is persistent.
			if (InParticle->IsDynamic() && !InParticle->IsSleeping())
			{
				// First time we have seen this particle, so add it
				check(!bLocked);
				ItemIndex = AddParticle(InParticle);
				InParticle->SetSolverBodyIndex(ItemIndex);
			}
			else // Not Dynamic
			{
				int32* ItemIndexPtr = ParticleToIndexMap.Find(InParticle);
				if (ItemIndexPtr != nullptr)
				{
					ItemIndex = *ItemIndexPtr;
				}
				else
				{
					// First time we have seen this particle, so add it
					check(!bLocked);
					ItemIndex = AddParticle(InParticle);
					ParticleToIndexMap.Add(InParticle, ItemIndex);
				}
			}			
		}
	
		check(ItemIndex != INDEX_NONE);
		check(ItemIndex < SolverBodies.Num());
		
		FSolverBody* SolverBody = &SolverBodies[ItemIndex];

		return SolverBody;
	}

	void FSolverBodyContainer::GatherInput(const FReal Dt, const int32 BeginIndex, const int32 EndIndex)
	{
		check((BeginIndex >= 0) && (BeginIndex <= EndIndex));
		check((EndIndex >= 0) && (EndIndex <= SolverBodies.Num()));

		for (int32 SolverBodyIndex = BeginIndex; SolverBodyIndex < EndIndex; ++SolverBodyIndex)
		{
			UpdateSolverBodyFromParticle(SolverBodies[SolverBodyIndex], Particles[SolverBodyIndex], Dt);
		}
	}

	void FSolverBodyContainer::ScatterOutput(const int32 BeginIndex, const int32 EndIndex)
	{
		check((BeginIndex >= 0) && (BeginIndex <= EndIndex));
		check((EndIndex >= 0) && (EndIndex <= Num()));

		for (int32 SolverBodyIndex = BeginIndex; SolverBodyIndex < EndIndex; ++SolverBodyIndex)
		{
			if (SolverBodies[SolverBodyIndex].IsDynamic())
			{
				SolverBodies[SolverBodyIndex].ApplyCorrections();
				UpdateParticleFromSolverBody(Particles[SolverBodyIndex], SolverBodies[SolverBodyIndex]);
			}
		}
	}

	void FSolverBodyContainer::SetImplicitVelocities(FReal Dt)
	{
		for (FSolverBody& SolverBody : SolverBodies)
		{
			SolverBody.SetImplicitVelocity(Dt);
		}
	}

	void FSolverBodyContainer::ApplyCorrections()
	{
		for (FSolverBody& SolverBody : SolverBodies)
		{
			SolverBody.ApplyCorrections();
		}
	}

	void FSolverBodyContainer::UpdateRotationDependentState()
	{
		for (FSolverBody& SolverBody : SolverBodies)
		{
			SolverBody.UpdateRotationDependentState();
		}
	}

}