// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/Evolution/SolverPartitionManager.h"
#include "Chaos/Evolution/SolverConstraintContainer.h"
#include "Chaos/ConstraintHandle.h"
#include "Chaos/ParticleHandle.h"

#include "GenericPlatform/GenericPlatformMath.h"

namespace Chaos
{
	namespace CVars
	{
		int32 Chaos_SolverPartitionManager_Partitioning = 0;
		FAutoConsoleVariableRef CVarChaosSolverPartitionManagerPartitioning(TEXT("p.Chaos.Solver.PartitionManager.Partitioning"), Chaos_SolverPartitionManager_Partitioning, TEXT("Create partitions for the constraint solver, 0: off, 1: greedy graph coloring."));

		int32 Chaos_SolverPartitionManager_ParallelSolve = -1;
		FAutoConsoleVariableRef CVarChaosSolverPartitionManagerParallelMode(TEXT("p.Chaos.Solver.PartitionManager.ParallelSolve"), Chaos_SolverPartitionManager_ParallelSolve, TEXT("Execute the constraint solver in parallel, -1: default (i.e. off without partitioning, on with partitioning), 0: off, 1: on (task-graph parallel)."), ECVF_Cheat);

		int32 Chaos_SolverPartitionManager_MinBatchSize = 200;
		FAutoConsoleVariableRef CVarChaosSolverPartitionManagerBatchSize(TEXT("p.Chaos.Solver.PartitionManager.MinBatchSize"), Chaos_SolverPartitionManager_MinBatchSize, TEXT("Minimum number of constraints or bodies per solver batch."));

		int32 Chaos_SolverPartitionManager_MinConstraintsForColoring = 1000;
		FAutoConsoleVariableRef CVarChaosSolverPartitionManagerMinConstraintsForColoring(TEXT("p.Chaos.Solver.PartitionManager.MinConstraintsForColoring"), Chaos_SolverPartitionManager_MinConstraintsForColoring, TEXT("Minimum number of constraints per constraint type needed to activate coloring."));
	}

	namespace Private
	{
		FSolverPartitionManager::FSolverPartitionManager(Chaos::FConstraintContainerSolver& ContainerIn, FPasskey Passkey)
			: Container(ContainerIn)
		{
		}

		FSolverPartitionManager::~FSolverPartitionManager()
		{
		}

		const TArray<TArray<int32>>& FSolverPartitionManager::GetConstraintsBatchIndices() const
		{
			return ConstraintsBatchIndices;
		}

		int32 FGreedyConstraintColoring::PreparePartitions()
		{
			Reset();
			ColorConstraints();
			AssignParallelBatchIndices();
			return ColorStartIndices.Num();
		}

		FGreedyConstraintColoring::FGreedyConstraintColoring(Chaos::FConstraintContainerSolver& ContainerIn, FPasskey Passkey)
			: FSolverPartitionManager(ContainerIn, Passkey)
		{
		}

		void FGreedyConstraintColoring::ColorConstraints()
		{
			const int32 NumConstraints = Container.GetNumConstraints();
			const bool bIsPartitioningSupported = Container.SupportsPartitioning();

			// Do not color if partitioning not supported for this container type or if there are too few constraints in the container.
			if (!bIsPartitioningSupported || NumConstraints < CVars::Chaos_SolverPartitionManager_MinConstraintsForColoring)
			{
				return;
			}

			QUICK_SCOPE_CYCLE_COUNTER(STAT_GreedyConstraintColoring_ColorConstraints);

			// Collects all particles constrained to the constraints in the container
			// 2 particles per constraints if implemented for this container type
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_GreedyConstraintColoring_GetParticlesFromConstraintContainer);

				Container.GetAllConstrainedParticles(ConstrainedParticles);
				check(ConstrainedParticles.Num() % 2 == 0);
			}

			// Dereference each particle once, store ParticleIdx directly.
			// Particle->ParticleIdx is SOA-local (not globally unique across SOAs), so particles from different SOAs can alias. 
			// Coloring is never incorrect but can lead to less parallelism if particles from different SOAs with the same ParticleIdx are connect by a constraint.
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_GreedyConstraintColoring_GetBodyIndices);

				BodyIndices.Reserve(ConstrainedParticles.Num());
				int32 MaxDynamicParticleIdx = INDEX_NONE;
				for (const FGeometryParticleHandle* Particle : ConstrainedParticles)
				{
					if (Particle)
					{
						const FPBDRigidParticleHandle* Rigid = Particle->CastToRigidParticle();
						// Only dynamic, non-sleeping particles will be updated by a constraint and need to considered for coloring.
						if (Rigid && Rigid->IsDynamic() && !Rigid->IsSleeping())
						{
							const int32 SOAIdx = Particle->ParticleIdx;
							MaxDynamicParticleIdx = FMath::Max(MaxDynamicParticleIdx, SOAIdx);
							BodyIndices.Add(SOAIdx);
							continue;
						}
					}
					BodyIndices.Add(INDEX_NONE);
				}
				check(BodyIndices.Num() % 2 == 0);

				BodyColors.SetNumUninitialized(MaxDynamicParticleIdx + 1);
				FMemory::Memzero(BodyColors.GetData(), sizeof(uint64) * BodyColors.Num());
			}

			// Determine color for each constraint by looking for the smallest available color indices not carried by the connected bodies.
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_GreedyConstraintColoring_DetermineColor);

				ConstraintColorIndices.Reserve(NumConstraints);
				CountColors.Reserve(MaxColors + 1);

				for (int32 SlotIdx = 0; SlotIdx < BodyIndices.Num(); SlotIdx += 2)
				{
					const int32 BodyIdx0 = BodyIndices[SlotIdx];
					const int32 BodyIdx1 = BodyIndices[SlotIdx + 1];

					// BodyIdx == INDEX_NONE means the body is not-dynamic or sleeping.
					// It will not be updated by the constraint and does not need to be colored.
					uint64* BodyColors0 = (BodyIdx0 > INDEX_NONE) ? &BodyColors[BodyIdx0] : nullptr;
					uint64* BodyColors1 = (BodyIdx1 > INDEX_NONE) ? &BodyColors[BodyIdx1] : nullptr;

					// Greedy coloring: combine both particles' used-color bit masks and find the lowest free bit.
					// 4-bit example: 
					//  - Body0 already connected to constraint of color 2 => 0100
					//  - Body1 already connected to constraints of color 0 and 3 => 1001
					//  - Combined UsedColorsMask => 0100 | 1001 => 1101
					//  - Combined FreeColorsMask => ~1101       => 0010
					//  - CountTrailingZeros returns 1 => color 1 is still available for both bodies
					const uint64 UsedColorsMask = (BodyColors0 ? *BodyColors0 : 0) | (BodyColors1 ? *BodyColors1 : 0);
					const uint64 FreeColorsMask = ~UsedColorsMask;

					// If >MaxColors constraints share this particle, we have run out of space in the bitmask.
					// UsedColorsMask == 0xFFFFFFFFFFFFFFFF and FreeColorsMask == 0x0000000000000000.
					// The constraint will be added to MaxColors (now 64) which will be processed in series by the solver.
					const int32 ColorIndex = FreeColorsMask != 0 ? (int32)FPlatformMath::CountTrailingZeros64(FreeColorsMask) : MaxColors;

					// Add color index to a contiguous TArray.
					ConstraintColorIndices.Add(ColorIndex);

					// Mark the chosen color as taken for both connected bodies.
					if (ColorIndex < MaxColors)
					{
						// Set the i-th right-most bit to 1 for ColorIndex = i.
						const uint64 ColorBit = uint64(1) << ColorIndex;
						if (BodyColors0)
						{
							*BodyColors0 |= ColorBit;
						}
						if (BodyColors1)
						{
							*BodyColors1 |= ColorBit;
						}
					}

					// Keep track of the highest color index and how often every color appears.
					if (ColorIndex >= CountColors.Num())
					{
						// Add a counter for a new color
						CountColors.Add(1);
					}
					else
					{
						// Count up for an existing color
						++CountColors[ColorIndex];
					}
				}
			}

			// Find the start index of each color in the array of sorted constraints.
			const int32 NumColors = CountColors.Num();
			ColorStartIndices.SetNumUninitialized(NumColors);
			if (NumColors > 0)
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_GreedyConstraintColoring_DetermineStartIndices);

				int32 ColorStartIndex = 0;
				ColorStartIndices[0] = ColorStartIndex;
				for (int32 ColorIndex = 1; ColorIndex < ColorStartIndices.Num(); ++ColorIndex)
				{
					ColorStartIndex += CountColors[ColorIndex - 1];
					ColorStartIndices[ColorIndex] = ColorStartIndex;
				}
			}

			// Sort the constraints according to their colors indices while maintaining relative order.
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_GreedyConstraintColoring_SortConstraints);
				check(!ConstraintColorIndices.IsEmpty());
				check(!ColorStartIndices.IsEmpty());
				if (!Container.SortConstraintsByPartitionIndex(ConstraintColorIndices, ColorStartIndices))
				{
					// If the sort function is not implemented for this container type, do not parallelize and flush out ColorStartIndices.
					ColorStartIndices.Reset();
				}
			}
		}

		void FGreedyConstraintColoring::AssignParallelBatchIndices()
		{
			if (!ColorStartIndices.IsEmpty())
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_GreedyConstraintColoring_AssignParallelBatchIndices);

				ConstraintsBatchIndices.SetNum(ColorStartIndices.Num());

				const int32 NumConstraints = Container.GetNumConstraints();
				const int32 MinConstraintsPerTask = CVars::Chaos_SolverPartitionManager_MinBatchSize;
				const int32 NumWorkerThreads = (FApp::ShouldUseThreadingForPerformance() && !bSingleWorkerPhysics) ? FMath::Min(FTaskGraphInterface::Get().GetNumWorkerThreads(), Chaos::MaxNumWorkers) : 1;

				for (int32 SerialIndex = 0; SerialIndex < ColorStartIndices.Num(); ++SerialIndex)
				{
					check(NumConstraints >= SerialIndex);
					if (SerialIndex >= MaxColors)
					{
						// Make sure this the max SerialIndex if we exceed MaxColors.
						check(SerialIndex == ColorStartIndices.Num() - 1);
						ConstraintsBatchIndices[SerialIndex].Reset(1);
						ConstraintsBatchIndices[SerialIndex].Add(ColorStartIndices[SerialIndex]);
						return;
					}

					const int32 SerialStartIndex = ColorStartIndices[SerialIndex];
					const int32 SerialEndIndex = SerialIndex < (ColorStartIndices.Num() - 1) ? (ColorStartIndices[SerialIndex + 1]) : (NumConstraints);
					const int32 NumConstraintsPerParallelBatch = SerialEndIndex - SerialStartIndex;
					const int32 ConstraintsByTask = FMath::Max(FMath::DivideAndRoundUp(NumConstraintsPerParallelBatch, NumWorkerThreads), MinConstraintsPerTask);
					const int32 NumConstraintBatches = FMath::Max(FMath::DivideAndRoundUp(NumConstraintsPerParallelBatch, ConstraintsByTask), 1);

					ConstraintsBatchIndices[SerialIndex].Reset(NumConstraintBatches);

					for (int32 ParallelIndex = 0; ParallelIndex < NumConstraintBatches; ++ParallelIndex)
					{
						const int32 BatchStartIndex = SerialStartIndex + ParallelIndex * ConstraintsByTask;
						check(NumConstraints >= BatchStartIndex);

						// Assign start indices of all parallel batches of this serial batch
						ConstraintsBatchIndices[SerialIndex].Add(BatchStartIndex);
					}
				}
			}
		}

		void FGreedyConstraintColoring::Reset()
		{
			ConstrainedParticles.Reset();
			BodyIndices.Reset();
			BodyColors.Reset();
			ConstraintColorIndices.Reset();
			CountColors.Reset();
			ColorStartIndices.Reset();
			ConstraintsBatchIndices.Reset();
		}
	}
}