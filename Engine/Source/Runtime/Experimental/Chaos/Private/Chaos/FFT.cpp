// Copyright Epic Games, Inc. All Rights Reserved.
// Parts of this file are adapted from PhysBAM under Copyright http://physbam.stanford.edu/links/backhistdisclaimcopy.html
#include "Chaos/FFT.h"

namespace Chaos
{
	void ConjugateAll(const FFFT3::FUniformGrid& Grid, FFFT3::FArrayNDOfComplex& Velocity, const int32 z)
	{
		for(int i = 1; i < Grid.Counts()[0] / 2; ++i)
		{
			Velocity(Grid.Counts()[0] - i, 0, z) = Velocity(i, 0, z).Conjugated();
			Velocity(Grid.Counts()[0] - i, Grid.Counts()[1] / 2, z) = Velocity(i, Grid.Counts()[1] / 2, z).Conjugated();
		}
		for(int i = 1; i < Grid.Counts()[1] / 2; ++i)
		{
			Velocity(0, Grid.Counts()[1] - i, z) = Velocity(0, i, z).Conjugated();
		}
		for(int i = 1; i < Grid.Counts()[0]; ++i)
			for(int j = 1; j < Grid.Counts()[1] / 2; ++j)
			{
				Velocity(Grid.Counts()[0] - i, Grid.Counts()[1] - j, z) = Velocity(i, j, z).Conjugated();
			}
	}
	void EnforceSymmetry(const FFFT3::FUniformGrid& Grid, FFFT3::FArrayNDOfComplex& Velocity)
	{
		Velocity[0].MakeReal();
		Velocity(Grid.Counts()[0] / 2, 0, 0).MakeReal();
		Velocity(0, Grid.Counts()[1] / 2, 0).MakeReal();
		Velocity(Grid.Counts()[0] / 2, Grid.Counts()[1] / 2, 0).MakeReal();
		Velocity(0, 0, Grid.Counts()[2] / 2).MakeReal();
		Velocity(Grid.Counts()[0] / 2, 0, Grid.Counts()[2] / 2).MakeReal();
		Velocity(0, Grid.Counts()[1] / 2, Grid.Counts()[2] / 2).MakeReal();
		Velocity(Grid.Counts()[0] / 2, Grid.Counts()[1] / 2, Grid.Counts()[2] / 2).MakeReal();
		ConjugateAll(Grid, Velocity, 0);
		ConjugateAll(Grid, Velocity, Grid.Counts()[2] / 2);
	}
	void FFFT3::MakeDivergenceFree(const FUniformGrid& Grid, FArrayNDOfComplex& u, FArrayNDOfComplex& v, FArrayNDOfComplex& w)
	{
		FVec3 Coefficients = ((FReal)2 * (FReal)UE_PI) / Grid.DomainSize();
		for(int i = 1; i <= Grid.Counts()[0] / 2; ++i)
			u(i, 0, 0) = FComplex(0, 0);
		for(int i = 0; i < Grid.Counts()[0]; ++i)
		{
			FReal k1 = Coefficients[0] * (i <= Grid.Counts()[0] / 2 ? (FReal)i : (FReal)i - (FReal)(Grid.Counts()[0]));
			for(int j = 1; j <= Grid.Counts()[1] / 2; ++j)
			{
				FReal k2 = Coefficients[1] * (FReal)j;
				FReal OneOverKSquared = 1 / (k1 * k1 + k2 * k2);
				FComplex correction = (k1 * u(i, j, 0) + k2 * v(i, j, 0)) * OneOverKSquared;
				u(i, j, 0) -= correction * k1;
				v(i, j, 0) -= correction * k2;
			}
		}
		// volume
		for(int i = 0; i < Grid.Counts()[0]; ++i)
		{
			FReal k1 = Coefficients[0] * (i <= Grid.Counts()[0] / 2 ? (FReal)i : (FReal)i - (FReal)Grid.Counts()[0]);
			for(int j = 0; j < Grid.Counts()[1]; ++j)
			{
				FReal k2 = Coefficients[1] * (j <= Grid.Counts()[1] / 2 ? (FReal)j : (FReal)j - (FReal)Grid.Counts()[1]);
				for(int k = 1; k <= Grid.Counts()[2] / 2; ++k)
				{
					FReal k3 = Coefficients[2] * (FReal)k;
					FReal OneOverKSquared = 1 / (k1 * k1 + k2 * k2 + k3 * k3);
					FComplex correction = (k1 * u(i, j, k) + k2 * v(i, j, k) + k3 * w(i, j, k)) * OneOverKSquared;
					u(i, j, k) -= correction * k1;
					v(i, j, k) -= correction * k2;
					w(i, j, k) -= correction * k3;
				}
			}
		}
		EnforceSymmetry(Grid, u);
		EnforceSymmetry(Grid, v);
		EnforceSymmetry(Grid, w);
	}

	// Numerical Receipes
	void exchange(FReal& f1, FReal& f2)
	{
		FReal tmp = f1;
		f1 = f2;
		f2 = tmp;
	}

	template<int d>
	void NRFourn(const int isign, const TVector<int32, d>& counts, TArray<FReal>& Data)
	{
		int idim;
		unsigned long i1, i2, i3, i2rev, i3rev, ip1, ip2, ip3, ifp1, ifp2;
		unsigned long ibit, k1, k2, n, nprev, nrem, ntot;
		FReal tempi, tempr;
		double theta, wi, wpi, wpr, wr, wtemp;

		ntot = counts.Product();
		nprev = 1;
		for(idim = d - 1; idim >= 0; idim--)
		{
			n = counts[idim];
			nrem = ntot / (n * nprev);
			ip1 = nprev << 1;
			ip2 = ip1 * n;
			ip3 = ip2 * nrem;
			i2rev = 1;
			for(i2 = 1; i2 <= ip2; i2 += ip1)
			{
				if(i2 < i2rev)
					for(i1 = i2; i1 <= i2 + ip1 - 2; i1 += 2)
						for(i3 = i1; i3 <= ip3; i3 += ip2)
						{
							i3rev = i2rev + i3 - i2;
							exchange(Data[i3 - 1], Data[i3rev - 1]);
							exchange(Data[i3], Data[i3rev]);
						}
				ibit = ip2 >> 1;
				while(ibit >= ip1 && i2rev > ibit)
				{
					i2rev -= ibit;
					ibit >>= 1;
				}
				i2rev += ibit;
			}
			ifp1 = ip1;
			while(ifp1 < ip2)
			{
				ifp2 = ifp1 << 1;
				theta = isign * 6.28318530717959 / (FReal)(ifp2 / ip1);
				wtemp = sin(.5 * theta);
				wpr = -2 * wtemp * wtemp;
				wpi = sin(theta);
				wr = 1;
				wi = 0;
				for(i3 = 1; i3 <= ifp1; i3 += ip1)
				{
					for(i1 = i3; i1 <= i3 + ip1 - 2; i1 += 2)
						for(i2 = i1; i2 <= ip3; i2 += ifp2)
						{
							k1 = i2;
							k2 = k1 + ifp1;
							tempr = (FReal)wr * Data[k2 - 1] - (FReal)wi * Data[k2];
							tempi = (FReal)wr * Data[k2] + (FReal)wi * Data[k2 - 1];
							Data[k2 - 1] = Data[k1 - 1] - tempr;
							Data[k2] = Data[k1] - tempi;
							Data[k1 - 1] += tempr;
							Data[k1] += tempi;
						}
					wr = (wtemp = wr) * wpr - wi * wpi + wr;
					wi = wi * wpr + wtemp * wpi + wi;
				}
				ifp1 = ifp2;
			}
			nprev *= n;
		}
	}

	void InverseTransformHelper(const FFFT3::FUniformGrid& Grid, TArrayND<FVec3, 3>& Velocity, const FFFT3::FArrayNDOfComplex& u, const int32 index, const bool Normalize)
	{
		int32 Size = Grid.Counts().Product();
		TArray<FReal> Data;
		Data.SetNum(2 * Size);
		int32 k = 0;
		for(int32 i = 0; i < Grid.Counts()[0]; ++i)
		{
			int32 negi = (i == 0) ? 0 : Grid.Counts()[0] - i;
			for(int32 j = 0; j < Grid.Counts()[1]; ++j)
			{
				int32 negj = (j == 0) ? 0 : Grid.Counts()[1] - j;
				for(int32 ij = 0; ij <= Grid.Counts()[2] / 2; ++ij)
				{
					Data[k++] = (FReal)u(i, j, ij).Real();
					Data[k++] = (FReal)u(i, j, ij).Imaginary();
				}
				for(int32 ij = Grid.Counts()[2] / 2 + 1; ij < Grid.Counts()[2]; ++ij)
				{
					int32 negij = Grid.Counts()[2] - ij;
					Data[k++] = (FReal)u(negi, negj, negij).Real();
					Data[k++] = -(FReal)u(negi, negj, negij).Imaginary();
				}
			}
		}
		NRFourn<3>(1, Grid.Counts(), Data);
		k = 0;
		const FReal multiplier = Normalize ? ((FReal)1 / (FReal)Size) : 1;
		for(int32 i = 0; i < Grid.Counts()[0]; ++i)
		{
			for(int32 j = 0; j < Grid.Counts()[1]; ++j)
			{
				for(int32 ij = 0; ij < Grid.Counts()[2]; ++ij)
				{
					Velocity(i, j, ij)[index] = Data[k++] * multiplier;
					k++;
				}
			}
		}
	}

	void FFFT3::InverseTransform(const FUniformGrid& Grid, TArrayND<FVec3, 3>& Velocity, const FArrayNDOfComplex& u, const FArrayNDOfComplex& v, const FArrayNDOfComplex& w, const bool Normalize)
	{
		InverseTransformHelper(Grid, Velocity, u, 0, Normalize);
		InverseTransformHelper(Grid, Velocity, v, 1, Normalize);
		InverseTransformHelper(Grid, Velocity, w, 2, Normalize);
	}

	void TransformHelper(const FFFT3::FUniformGrid& Grid, const TArrayND<FVec3, 3>& Velocity, FFFT3::FArrayNDOfComplex& u, const int32 index)
	{
		int32 Size = Grid.Counts().Product();
		TArray<FReal> Data;
		Data.SetNum(2 * Size);
		int32 k = 0;
		for(int32 i = 0; i < Grid.Counts()[0]; ++i)
		{
			for(int32 j = 0; j < Grid.Counts()[1]; ++j)
			{
				for(int32 ij = 0; ij < Grid.Counts()[2]; ++ij)
				{
					Data[k++] = (FReal)Velocity(i, j, ij)[index];
					Data[k++] = 0.f;
				}
			}
		}
		NRFourn<3>(-1, Grid.Counts(), Data);
		k = 0;
		for(int32 i = 0; i < Grid.Counts()[0]; ++i)
		{
			for(int32 j = 0; j < Grid.Counts()[1]; ++j)
			{
				for(int32 ij = 0; ij <= Grid.Counts()[2] / 2; ++ij)
				{
					u(i, j, ij) = FComplex(Data[k], Data[k + 1]);
					k += 2;
				}
				k += Grid.Counts()[2] - 2;
			}
		}
	}
	void FFFT3::Transform(const FUniformGrid& Grid, const TArrayND<FVec3, 3>& Velocity, FArrayNDOfComplex& u, FArrayNDOfComplex& v, FArrayNDOfComplex& w)
	{
		TransformHelper(Grid, Velocity, u, 0);
		TransformHelper(Grid, Velocity, v, 1);
		TransformHelper(Grid, Velocity, w, 2);
	}
}
