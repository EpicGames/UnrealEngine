// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/Framework/ChaosResultsManager.h"
#include "PhysicsProxy/ClusterUnionPhysicsProxy.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "PhysicsProxy/GeometryCollectionPhysicsProxy.h"
#include "Chaos/ChaosMarshallingManager.h"
#include "Chaos/Framework/PhysicsSolverBase.h"

DECLARE_CYCLE_STAT(TEXT("Process Rigid Resim Targets"), STAT_RigidResimTargets, STATGROUP_Chaos);

namespace Chaos
{	
	enum class ESetPrevNextDataMode
	{
		Prev,
		Next,
	};

	/** Helper class used to interpolate results per channel */
	struct FChaosResultsChannel
	{
		FChaosResultsChannel(FChaosResultsManager& InResultsManager, int32 InChannelIdx)
		: ResultsManager(InResultsManager)
		, ChannelIdx(InChannelIdx)
		{
		}

		~FChaosResultsChannel();
		FChaosResultsChannel(const FChaosResultsChannel&) = delete;
		FChaosResultsChannel(FChaosResultsChannel&& Other) = default;

		const FChaosInterpolationResults& UpdateInterpAlpha_External(const FReal ResultsTime, const FReal GlobalAlpha);
		void ProcessResimResult_External();
		bool AdvanceResult();
		void CollapseResultsToLatest();
		const FChaosInterpolationResults& PullSyncPhysicsResults_External();
		const FChaosInterpolationResults& PullAsyncPhysicsResults_External(const FReal ResultsTime);

		template <ESetPrevNextDataMode Mode>
		void SetPrevNextDataHelper(const FPullPhysicsData& PullData);
		
		template <ESetPrevNextDataMode Mode, typename DirtyProxyDataType, typename InterpolationsType>
		void SetPrevNextDataHelperTyped(const TArray<DirtyProxyDataType>& DirtyProxies, TArray<InterpolationsType>& Interpolations);

		void RemoveProxy_External(FSingleParticlePhysicsProxy* Proxy)
		{
			ParticleToResimTarget.Remove(Proxy);
		}

		FChaosInterpolationResults Results;
		FReal LatestTimeSeen = 0;	//we use this to know when resim results are being pushed
		TMap<FSingleParticlePhysicsProxy*, FDirtyRigidParticleData> ParticleToResimTarget;
		FChaosResultsManager& ResultsManager;
		int32 ChannelIdx;
	};

	template <typename TProxyType, typename TInterpolationType>
	static void ResetInterpolations(TArray<TInterpolationType>& Interpolations)
	{
		for (TInterpolationType& Data : Interpolations)
		{
			if (TProxyType* Proxy = Data.Prev.GetProxy())
			{
				FProxyInterpolationBase* InterpolationData = Proxy->GetInterpolationData();
				if (InterpolationData)
				{
					InterpolationData->SetPullDataInterpIdx_External(INDEX_NONE);
				}
			}
		}
		Interpolations.Reset();
	}
	
	void FChaosInterpolationResults::Reset()
	{
		ResetInterpolations<FSingleParticlePhysicsProxy>(RigidInterpolations);
		ResetInterpolations<FGeometryCollectionPhysicsProxy>(GeometryCollectionInterpolations);
		ResetInterpolations<FClusterUnionPhysicsProxy>(ClusterUnionInterpolations);

		//purposely leave Prev and Next alone as we use those for rebuild
	}

	template <ESetPrevNextDataMode Mode, typename DirtyProxyDataType, typename InterpolationsType>
	void FChaosResultsChannel::SetPrevNextDataHelperTyped(const TArray<DirtyProxyDataType>& DirtyProxies, TArray<InterpolationsType>& Interpolations)
	{
		//clear results
		for (const DirtyProxyDataType& Data : DirtyProxies)
		{
			if (auto Proxy = Data.GetProxy())
			{
				FProxyInterpolationBase* InterpolationData = Proxy->GetInterpolationData();
				
				//If proxy is not associated with this channel, do nothing
				if (!InterpolationData || InterpolationData->GetInterpChannel_External() != ChannelIdx)
				{
					continue;
				}

				int32 DataIdx = InterpolationData->GetPullDataInterpIdx_External();
				if(DataIdx == INDEX_NONE)
				{
					DataIdx = Interpolations.AddDefaulted(1);
					InterpolationData->SetPullDataInterpIdx_External(DataIdx);

					if(Mode == ESetPrevNextDataMode::Next)
					{
						//no prev so use GT data
						Proxy->BufferPhysicsResults_External(Interpolations[DataIdx].Prev);
					}
				}

				InterpolationsType& OutData = Interpolations[DataIdx];

				if(Mode == ESetPrevNextDataMode::Prev)
				{
					//if particle doesn't change we won't get it in next step, so just interpolate as constant
					OutData.Prev = Data;
					OutData.Next = Data;
				}
				else if(Mode == ESetPrevNextDataMode::Next)
				{
					OutData.Next = Data;
				}
			}
		}
	}
	
	template <ESetPrevNextDataMode Mode>
	void FChaosResultsChannel::SetPrevNextDataHelper(const FPullPhysicsData& PullData)
	{
		//clear results
		const int32 Timestamp = PullData.SolverTimestamp;
		
		{
			SCOPE_CYCLE_COUNTER(STAT_ProcessSingleProxy);
			SetPrevNextDataHelperTyped<Mode>(PullData.DirtyRigids, Results.RigidInterpolations);
		}

		{
			SCOPE_CYCLE_COUNTER(STAT_ProcessGCProxy);
			SetPrevNextDataHelperTyped<Mode>(PullData.DirtyGeometryCollections, Results.GeometryCollectionInterpolations);
		}

		{
			SCOPE_CYCLE_COUNTER(STAT_ProcessClusterUnionProxy);
			SetPrevNextDataHelperTyped<Mode>(PullData.DirtyClusterUnions, Results.ClusterUnionInterpolations);
		}

		// update resim target for rigids
		{
			SCOPE_CYCLE_COUNTER(STAT_RigidResimTargets);
			for(const FDirtyRigidParticleData& Data : PullData.DirtyRigids)
			{
				if(FSingleParticlePhysicsProxy* Proxy = Data.GetProxy())
				{
					FProxyInterpolationBase* InterpolationData = Proxy->GetInterpolationData();
					// only if the proxy is associated with this channel 
					if(InterpolationData && InterpolationData->GetInterpChannel_External() == ChannelIdx)
					{
						//update leash target
						if(FDirtyRigidParticleData* ResimTarget = ParticleToResimTarget.Find(Proxy))
						{
							*ResimTarget = Data;
						}
					}
				}
			}
		}
	}

	/** 
	 Advance the results in the marshaller queue by one if it is available 

	 @param MarshallingManager Manger to advance
	 @param Results Results to update with advanced state
	 @return whether an advance occurred
	*/
	bool FChaosResultsChannel::AdvanceResult()
	{
		if(FPullPhysicsData* PotentialNext = ResultsManager.PopPullData_External(ChannelIdx))
		{
			//newer result exists so prev is overwritten
			if(Results.Prev)
			{
				ResultsManager.FreePullData_External(Results.Prev, ChannelIdx);
			}

			Results.Prev = Results.Next;
			//mark prev with next's data.
			//any particles that were dirty in the previous results and are now constant will still have the old values set
			SetPrevNextDataHelper<ESetPrevNextDataMode::Prev>(*Results.Prev);

			Results.Next = PotentialNext;

			// Move any non applied DirtyRigidErrors from previous to next
			if (Results.Prev->DirtyRigidErrors.Num() > 0)
			{
				Results.Next->DirtyRigidErrors.Append(Results.Prev->DirtyRigidErrors);
			}
			
			if(PotentialNext->ExternalEndTime <= LatestTimeSeen)
			{
				//this must be the results of a resim, so compare it to original results for divergence
				ProcessResimResult_External();
			}

			LatestTimeSeen = FMath::Max(LatestTimeSeen, PotentialNext->ExternalEndTime);
			return true;
		}
		
		return false;
	}

	/**
	 Collapse the whole pending queue inside a marshalling manager to one results object written to Results.Next

	 @param MarshallingManager Manger to advance
	 @param Results Results to update with advanced state
	*/
	void FChaosResultsChannel::CollapseResultsToLatest()
	{
		if(Results.Next == nullptr)
		{
			//nothing in Next (first time), so get latest if possible
			Results.Next = ResultsManager.PopPullData_External(ChannelIdx);
		}

		while(AdvanceResult())
		{}
	}

	const FChaosInterpolationResults& FChaosResultsManager::PullSyncPhysicsResults_External()
	{
		return Channels[0]->PullSyncPhysicsResults_External();
	}

	const FChaosInterpolationResults& FChaosResultsChannel::PullSyncPhysicsResults_External()
	{
		//sync mode doesn't use prev results, but if we were async previously we need to clean it up
		if (Results.Prev)
		{
			ResultsManager.FreePullData_External(Results.Prev, ChannelIdx);
			Results.Prev = nullptr;
		}

		//either brand new, or we are consuming new results. Either way need to rebuild everything
		Results.Reset();

		// If we switched from async to sync we may have multiple pending results, so discard them all except latest.
		// If we dispatched substeps there will be multiple results pending but the latest is the one we want.
		CollapseResultsToLatest();

		if (Results.Next)
		{
			//whatever next ends up being, we mark the data as such
			SetPrevNextDataHelper<ESetPrevNextDataMode::Next>(*Results.Next);
			Results.Alpha = 1;
		}

		return Results;
	}

	FReal ComputeAlphaHelper(const FPullPhysicsData& Next, const FReal ResultsTime)
	{
		const FReal Denom = Next.ExternalEndTime - Next.ExternalStartTime;
		if (Denom > 0)	//if 0 dt just skip
		{
			//if we have no future results alpha will be > 1
			//in that case we just keep rendering the latest results we can
			return FMath::Min((FReal)1., (ResultsTime - Next.ExternalStartTime) / Denom);
		}

		return 1;	//if 0 dt just use 1 as alpha
	}

	FRealSingle SecondChannelDelay = 0.05f;
	FAutoConsoleVariableRef CVarSecondChannelDelay(TEXT("p.SecondChannelDelay"), SecondChannelDelay, TEXT(""));

	int32 DefaultNumActiveChannels = 1;
	FAutoConsoleVariableRef CVarNumActiveChannels(TEXT("p.NumActiveChannels"), DefaultNumActiveChannels, TEXT(""));

	FChaosResultsManager::FChaosResultsManager(FChaosMarshallingManager& InMarshallingManager)
		: MarshallingManager(InMarshallingManager)
		, NumActiveChannels(DefaultNumActiveChannels)
	{
		for(int32 Channel = 0; Channel < NumActiveChannels; ++Channel)
		{
			Channels.Emplace(new FChaosResultsChannel(*this, Channel));
		}

		PerChannelTimeDelay.Add(0);
		PerChannelTimeDelay.Add(SecondChannelDelay);
	}

	const FChaosInterpolationResults& FChaosResultsChannel::UpdateInterpAlpha_External(FReal ResultsTime, const FReal GlobalAlpha)
	{
		Results.Alpha = (float)GlobalAlpha;	 // LWC_TODO: Precision loss

		//make sure any resim interpolated bodies are still in the results array.
		//It's possible the body stopped moving after the resim and is not dirty, but we still want to interpolate to final place
		TArray<FSingleParticlePhysicsProxy*> FinishedSmoothing;
		for (const auto& Itr : ParticleToResimTarget)
		{
			FSingleParticlePhysicsProxy* Proxy = Itr.Key;

			FProxyInterpolationBase* InterpolationData = Proxy->GetInterpolationData();
			if (InterpolationData && InterpolationData->IsErrorSmoothing())
			{
				if (InterpolationData->GetPullDataInterpIdx_External() == INDEX_NONE)	//not in results array
				{
					//still need to interpolate, so add to results array
					const int32 DataIdx = Results.RigidInterpolations.AddDefaulted(1);
					InterpolationData->SetPullDataInterpIdx_External(DataIdx);
					FChaosRigidInterpolationData& RigidData = Results.RigidInterpolations[DataIdx];

					RigidData.Next = Itr.Value;			//not dirty from sim, so just use whatever last next was
					RigidData.Prev = RigidData.Next;	//prev same as next since we're just using leash
				}
			}
			else
			{
				FinishedSmoothing.Add(Proxy);
			}
		}

		for(FSingleParticlePhysicsProxy* Proxy : FinishedSmoothing)
		{
			RemoveProxy_External(Proxy);
		}

		return Results;
	}

	TArray<const FChaosInterpolationResults*> FChaosResultsManager::PullAsyncPhysicsResults_External(const FReal ResultsTime)
	{
		TArray<const FChaosInterpolationResults*> InterpResults;
		for(int32 ChannelIdx = 0; ChannelIdx < NumActiveChannels; ++ChannelIdx)
		{
			InterpResults.Add(&Channels[ChannelIdx]->PullAsyncPhysicsResults_External(ResultsTime - PerChannelTimeDelay[ChannelIdx]));
		}

		return InterpResults;
	}

	const FChaosInterpolationResults& FChaosResultsChannel::PullAsyncPhysicsResults_External(const FReal ResultsTime)
	{
		//in async mode we must interpolate between Start and End of a particular sim step, where ResultsTime is in the inclusive interval [Start, End]
		//to do this we need to keep the results of the previous sim step, which ends exactly when the next one starts
		//if no previous result exists, we use the existing GT data

		if(ResultsTime < 0)
		{
			return UpdateInterpAlpha_External(ResultsTime, 1);
		}


		//still need previous results, so rebuild them
		if (Results.Next && ResultsTime <= Results.Next->ExternalEndTime)
		{
			//already have results, just need to update alpha
			const FReal GlobalAlpha = ComputeAlphaHelper(*Results.Next, ResultsTime);
			return UpdateInterpAlpha_External(ResultsTime, GlobalAlpha);
		}

		//either brand new, or we are consuming new results. Either way need to rebuild everything
		Results.Reset();

		if (Results.Next == nullptr)
		{
			//nothing in Next (first time), so get latest if possible
			Results.Next = ResultsManager.PopPullData_External(ChannelIdx);
		}

		if(Results.Next)
		{
			//go through every result and record the dirty proxies
			while (Results.Next->ExternalEndTime < ResultsTime)
			{
				if(!AdvanceResult())
				{
					break;
				}
			}
		}

		ensure(Results.Prev == nullptr || Results.Next != nullptr);	//we can never have a prev set when there isn't a next

		FReal GlobalAlpha = 1;
		if(Results.Next)
		{
			//whatever next ends up being, we mark the data as such
			SetPrevNextDataHelper<ESetPrevNextDataMode::Next>(*Results.Next);
			GlobalAlpha = ComputeAlphaHelper(*Results.Next, ResultsTime);
		}

		return UpdateInterpAlpha_External(ResultsTime, GlobalAlpha);
	}

	bool StateDiverged(const FDirtyRigidParticleData& A, const FDirtyRigidParticleData& B)
	{
		ensure(A.GetProxy() == B.GetProxy());
		return A.X != B.X || A.R != B.R || A.V != B.V || A.W != B.W || A.ObjectState != B.ObjectState;
	}

	void FChaosResultsChannel::ProcessResimResult_External()
	{
		//make sure any proxy in the resim data is marked as resimming
		for(const FDirtyRigidParticleData& ResimDirty : Results.Next->DirtyRigids)
		{
			if (FSingleParticlePhysicsProxy* ResimProxy = ResimDirty.GetProxy())
			{
				FProxyInterpolationBase* InterpolationData = ResimProxy->GetInterpolationData(); 
				//Mark as resim only if proxy is owned by this channel
				if(InterpolationData && InterpolationData->GetInterpChannel_External() == ChannelIdx)
				{
					ParticleToResimTarget.FindOrAdd(ResimProxy) = ResimDirty;
				}
			}
		}
	}

	FChaosResultsChannel::~FChaosResultsChannel()
	{
		if(Results.Prev)
		{
			ResultsManager.FreePullData_External(Results.Prev, ChannelIdx);
		}

		if(Results.Next)
		{
			ResultsManager.FreePullData_External(Results.Next, ChannelIdx);
		}
	}

	void FChaosResultsManager::RemoveProxy_External(FSingleParticlePhysicsProxy* Proxy)
	{
		for(FChaosResultsChannel* Channel : Channels)
		{
			Channel->RemoveProxy_External(Proxy);
		}
	}

	FPullPhysicsData* FChaosResultsManager::PopPullData_External(int32 ChannelIdx)
	{
		int32 FoundToPop = INDEX_NONE;
		for(int32 Idx = InternalQueue.Num() - 1; Idx >= 0; --Idx)
		{
			FPullDataQueueInfo& Info = InternalQueue[Idx];
			if(!Info.bHasPopped[ChannelIdx])
			{
				FoundToPop = Idx;
			}
			else
			{
				//Found an entry that was popped so stop searching
				break;
			}
		}

		if(FoundToPop == INDEX_NONE)
		{
			//Need to pop from main queue
			if(FPullPhysicsData* PullData = MarshallingManager.PopPullData_External())
			{
				FoundToPop = InternalQueue.Add(FPullDataQueueInfo(PullData, NumActiveChannels));
			}
			else
			{
				//Need to pop but nothing at head so return null
				return nullptr;
			}
		}

		FPullDataQueueInfo& Info = InternalQueue[FoundToPop];
		Info.bHasPopped[ChannelIdx] = true;
		return Info.PullData;
	}

	void FChaosResultsManager::FreePullData_External(FPullPhysicsData* PullData, int32 GivenChannelIdx)
	{
		for(int32 Idx = 0; Idx < InternalQueue.Num(); ++Idx)
		{
			FPullDataQueueInfo& Info = InternalQueue[Idx];
			if(Info.PullData == PullData)
			{
				ensure(Info.bHasPopped[GivenChannelIdx]);	//free before popped?
				ensure(Info.bPendingFree[GivenChannelIdx] == false);	//double free?

				Info.bPendingFree[GivenChannelIdx] = true;

				bool bFree = true;
				for(int32 ChannelIdx = 0; ChannelIdx < MaxNumChannels; ++ChannelIdx)
				{
					if(!Info.bPendingFree[ChannelIdx])
					{
						bFree = false;
						break;
					}
				}

				if(bFree)
				{
					MarshallingManager.FreePullData_External(Info.PullData);
					InternalQueue.RemoveAt(Idx);
				}

				return;
			}
		}

		ensure(false);	//didn't find queue entry, double free?
	}

	FChaosResultsManager::~FChaosResultsManager()
	{
		//first clean up channels
		for(FChaosResultsChannel* Channel : Channels)
		{
			delete Channel;
		}

		//if anything is left in the internal queue (for example channel was falling behind and never popped or freed) clear it
		for(FPullDataQueueInfo& Info : InternalQueue)
		{
			MarshallingManager.FreePullData_External(Info.PullData);
		}
	}
}
