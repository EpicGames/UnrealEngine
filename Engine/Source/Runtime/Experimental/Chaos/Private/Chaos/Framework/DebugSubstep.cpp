// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/Framework/DebugSubstep.h"

#if CHAOS_DEBUG_SUBSTEP

#include "ChaosLog.h"
#include "HAL/Event.h"
#include "HAL/PlatformProcess.h"

namespace Chaos
{
	FDebugSubstep::FDebugSubstep()
		: bIsEnabled(false)
		, CommandQueue()
		, ProgressEvent(nullptr)
		, SubstepEvent(nullptr)
		, ThreadId(0)
		, bWaitForStep(false)
		, bShouldEnable(false)
	{}

	FDebugSubstep::~FDebugSubstep()
	{
		Release();
	}

	void FDebugSubstep::Initialize()
	{
		if (!IsInitialized())
		{
			// Allocate sync events
			ProgressEvent = FPlatformProcess::GetSynchEventFromPool();
			SubstepEvent = FPlatformProcess::GetSynchEventFromPool(true);  // SubstepEvent can be triggered without a matching wait, hence the manual reset setting
		}
	}

	void FDebugSubstep::Release()
	{
		if (IsInitialized())
		{
			// Exit debug thread
			if (bIsEnabled)
			{
				Stop();
			}

			// Release sync events
			FPlatformProcess::ReturnSynchEventToPool(ProgressEvent);
			FPlatformProcess::ReturnSynchEventToPool(SubstepEvent);
			ProgressEvent = nullptr;
			SubstepEvent = nullptr;

			// Reset commands
			CommandQueue.Empty();
			bShouldEnable = false;
		}
	}

	void FDebugSubstep::Shutdown()
	{
		// Exit debug thread
		if (IsInitialized() && bIsEnabled)
		{
			Stop();
		}
	}

	void FDebugSubstep::AssumeThisThread()
	{
		ThreadId = FPlatformTLS::GetCurrentThreadId();
	}

	void FDebugSubstep::Enable(bool bEnable)
	{
		CommandQueue.Enqueue(bEnable ? ECommand::Enable: ECommand::Disable);
		UE_LOGF(LogChaosThread, Verbose, "[Game Thread] Enable=%ls", bEnable ? TEXT("True"): TEXT("False"));
	}

	void FDebugSubstep::ProgressToSubstep()
	{
		CommandQueue.Enqueue(ECommand::ProgressToSubstep);
		UE_LOGF(LogChaosThread, Verbose, "[Game Thread] Progress");
	}

	void FDebugSubstep::ProgressToStep()
	{
		CommandQueue.Enqueue(ECommand::ProgressToStep);
		UE_LOGF(LogChaosThread, Verbose, "[Game Thread] Progress");
	}

	bool FDebugSubstep::SyncAdvance(bool bIsSolverEnabled)
	{
		// Don't process the commands unless this instance is initialized
		if (!IsInitialized()) { return false; }

		// Process all commands from the queue
		bool bShouldStep = false;
		bool bShouldSubstep = false;
		ECommand Command;
		while (CommandQueue.Dequeue(Command))
		{
			switch (Command)
			{
			case ECommand::Enable:
				bShouldEnable = true;
				break;

			case ECommand::Disable:
				bShouldEnable = false;
				break;

			case ECommand::ProgressToStep:
				bShouldStep = true;
				break;

			case ECommand::ProgressToSubstep:
				bShouldSubstep = true;
				break;

			default:
				UE_LOGF(LogChaosThread, Fatal, "Unknown debug step command.");
				break;
			}
		}

		if (!bIsEnabled)
		{
			// Currently disabled, check whether it needs enabling
			if (bShouldEnable && bIsSolverEnabled) // Can only allow the debug thread to run if the solver is already enabled.
			{
				// Start debug substep
				Start();
			}
		}
		else
		{
			// Currently enabled, check whether it needs disabling or stepping
			if (!bShouldEnable || !bIsSolverEnabled) // Should also stop the debug thread if the solver is disabled, otherwise it can't be re-enabled since the HandleSolverCommands won't get executed
			{
				// Stop debug substep
				Stop();
			}
			else if (bShouldStep || bShouldSubstep)
			{
				// Step/substep
				Substep(bShouldStep);
			}
		}

		return bIsEnabled;
	}

	void FDebugSubstep::Add(bool bInStep, const TCHAR* Label) const
	{
		// Ignore this substep unless this instance is initialized
		if (!IsInitialized()) { return; }

		// Manage events
		if (bIsEnabled)
		{
			if (bInStep)
			{
				// Signal a step boundary
				bWaitForStep = false;
			}
			checkf(ThreadId == FPlatformTLS::GetCurrentThreadId(), TEXT("Cannot add a substep outside of the solver thread (eg inside a parallel for)."));
			UE_LOGF(LogChaosThread, Log, "Reached %ls '%ls'", bInStep ? TEXT("step"): TEXT("substep"), Label ? Label: TEXT(""));
			UE_LOGF(LogChaosThread, Verbose, "[Debug Thread] Triggering substep event");
			SubstepEvent->Trigger();
			UE_LOGF(LogChaosThread, Verbose, "[Debug Thread] Waiting for progress event");
			ProgressEvent->Wait();
			UE_LOGF(LogChaosThread, Verbose, "[Debug Thread] Progress event received, wait ended");
		}
		else if (bInStep)  // Trigger one last event at the step boundary when disabled
		{
			UE_LOGF(LogChaosThread, Log, "Reached step '%ls'", Label ? Label: TEXT(""));
			UE_LOGF(LogChaosThread, Verbose, "[Debug Thread] Triggering substep event");
			SubstepEvent->Trigger();
		}
	}

	void FDebugSubstep::Start()
	{
		check(IsInitialized());
		check(!bIsEnabled);

		bIsEnabled = true;
		UE_LOGF(LogChaosThread, Verbose, "[%ls Thread] bIsEnabled changed (false->true)", IsInGameThread() ? TEXT("Game") : TEXT("Physics"));
		UE_LOGF(LogChaosThread, Log, "Chaos' debug substep mode is now engaged. Pausing solver thread at next step.");
		// Note: Once enabled the associated task must be created
	}

	void FDebugSubstep::Stop()
	{
		check(IsInitialized());
		check(bIsEnabled);

		bIsEnabled = false;
		UE_LOGF(LogChaosThread, Verbose, "[%ls Thread] bIsEnabled changed (true->false)", IsInGameThread() ? TEXT("Game") : TEXT("Physics"));

		// Trigger progress, with bIsEnabled being false it should go straight to the end of the step
		SubstepEvent->Reset();     // No race condition with the Add() between these two instructions
		ProgressEvent->Trigger();  // since this code path is only to be entered while in ProgressEvent->Wait(); state.

		// Wait for the final step event
		UE_LOGF(LogChaosThread, Verbose, "[%ls Thread] Waiting for last step event", IsInGameThread() ? TEXT("Game") : TEXT("Physics"));
		SubstepEvent->Wait();  // Last wait event will be triggered on debug thread exit

		UE_LOGF(LogChaosThread, Verbose, "[%ls Thread] Substep event received, wait ended", IsInGameThread() ? TEXT("Game") : TEXT("Physics"));
		UE_LOGF(LogChaosThread, Log, "Chaos' debug substep mode is now disengaged. Resuming solver thread at next step.");
		// Note: Once disabled the associated task must be deleted
	}

	void FDebugSubstep::Substep(bool bShouldStep)
	{
		check(IsInitialized());
		check(bIsEnabled);

		UE_LOGF(LogChaosThread, Verbose, "[%ls Thread] Triggering progress event", IsInGameThread() ? TEXT("Game") : TEXT("Physics"));
		bWaitForStep = bShouldStep;
		do
		{
			// Trigger progress
			SubstepEvent->Reset();     // No race condition with the Add() between these two instructions
			ProgressEvent->Trigger();  // since this code path is only to be entered while in ProgressEvent->Wait(); state.

			// Wait for next step/substep event
			UE_LOGF(LogChaosThread, Verbose, "[%ls Thread] Waiting for substep event", IsInGameThread() ? TEXT("Game") : TEXT("Physics"));
			SubstepEvent->Wait();
			UE_LOGF(LogChaosThread, Verbose, "[%ls Thread] Substep event received, wait ended", IsInGameThread() ? TEXT("Game") : TEXT("Physics"));
		} while (bWaitForStep);
	}
}

#endif  // #if CHAOS_DEBUG_SUBSTEP
