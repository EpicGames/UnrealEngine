// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/Framework/Parallel.h"
#include "Async/ParallelFor.h"
#include "Framework/Threading.h"
#include "ChaosDebugDraw/ChaosDDContext.h"

namespace Chaos
{
	// If GSingleThreadedPhysics is 1 physics simulation runs on the Game Thread in single thread 
	CHAOS_API int32 GSingleThreadedPhysics = 0;
	// If bSingleWorkerPhysics is true physics simulation run only on one thread on the Physics Thread
	CHAOS_API bool bSingleWorkerPhysics = false;
	CHAOS_API int32 InnerParallelForBatchSize = 0;
	CHAOS_API int32 MinRangeBatchSize = 0;
	CHAOS_API int32 MaxNumWorkers = 100;
	// Batch size for collision 
	// can be smaller in general because Narrow Phase between two rigid bodies can be a set of many shapes with height field or mesh. 
	CHAOS_API int32 CollisionSmallBatchSize = 10;
	// Batch size  for multi threading, use not to launch tasks if not enough instance
	CHAOS_API int32 GlobalSmallBatchSize = 40;
	CHAOS_API int32 LargeBatchSize = 100;
	
	CHAOS_API bool bDisablePhysicsParallelFor = false;
	CHAOS_API bool bDisableParticleParallelFor = false;
	CHAOS_API bool bDisableCollisionParallelFor = false;

	FAutoConsoleVariableRef CVarDisablePhysicsParallelFor(TEXT("p.Chaos.DisablePhysicsParallelFor"), bDisablePhysicsParallelFor, TEXT("Disable parallel execution in Chaos Evolution"));
	FAutoConsoleVariableRef CVarDisableParticleParallelFor(TEXT("p.Chaos.DisableParticleParallelFor"), bDisableParticleParallelFor, TEXT("Disable parallel execution for Chaos Particles (Collisions, "));
	FAutoConsoleVariableRef CVarDisableCollisionParallelFor(TEXT("p.Chaos.DisableCollisionParallelFor"), bDisableCollisionParallelFor, TEXT("Disable parallel execution for Chaos Collisions (also disabled by DisableParticleParallelFor)"));
	FAutoConsoleVariableRef CVarInnerPhysicsBatchSize(TEXT("p.Chaos.InnerParallelForBatchSize"), InnerParallelForBatchSize, TEXT("Set the batch size threshold for inner parallel fors"));
	FAutoConsoleVariableRef CVarMinRangeBatchSize(TEXT("p.Chaos.MinRangeBatchSize"), MinRangeBatchSize, TEXT("Set the min range batch size for parallel for"));
	FAutoConsoleVariableRef CVarMaxRangeBatchWorkers(TEXT("p.Chaos.MaxNumWorkers"), MaxNumWorkers, TEXT("Set the max number of workers for physics"));
	FAutoConsoleVariableRef CVarSmallBatchCollisionSize(TEXT("p.Chaos.CollisionSmallBatchSize"), CollisionSmallBatchSize, TEXT("Small batch size used for Collision Detection"));
	FAutoConsoleVariableRef CVarGlobalSmallBatchSize(TEXT("p.Chaos.GlobalSmallBatchSize"), GlobalSmallBatchSize, TEXT("Minimum batch size needed to launch multi-threaded tasks"));
	FAutoConsoleVariableRef CVarLargeBatchSize(TEXT("p.Chaos.LargeBatchSize"), LargeBatchSize, TEXT("Large batch size for chaos tasks"));
	FAutoConsoleVariableRef CVarSingleWorkerPhysics(TEXT("p.Chaos.SingleWorkerPhysics"), bSingleWorkerPhysics, TEXT("Run Chaos on a single thread"));
}

void Chaos::InnerPhysicsParallelFor(int32 Num, TFunctionRef<void(int32)> InCallable, bool bForceSingleThreaded)
{
	int32 NumWorkers = int32(LowLevelTasks::FScheduler::Get().GetNumWorkers());
	if (NumWorkers == 0)
	{
		NumWorkers = 1;
	}
	int32 BatchSize = FMath::DivideAndRoundUp<int32>(Num, NumWorkers);
	PhysicsParallelFor(Num, InCallable, (BatchSize > InnerParallelForBatchSize) ? bForceSingleThreaded : true);
}

void Chaos::InnerPhysicsParallelForRange(int32 InNum, TFunctionRef<void(int32, int32)> InCallable, const int32 InMinBatchSize, bool bForceSingleThreaded)
{
	int32 NumWorkers = int32(LowLevelTasks::FScheduler::Get().GetNumWorkers());
	if (NumWorkers == 0)
	{
		NumWorkers = 1;
	}
	int32 BatchSize = FMath::DivideAndRoundUp<int32>(InNum, NumWorkers);
	PhysicsParallelForRange(InNum, InCallable, InMinBatchSize, (BatchSize > InnerParallelForBatchSize) ? bForceSingleThreaded : true);
}

void Chaos::PhysicsParallelFor(int32 InNum, TFunctionRef<void(int32)> InCallable, bool bForceSingleThreaded)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Chaos_PhysicsParallelFor);
	using namespace Chaos;

	// Passthrough for now, except with global flag to disable parallel
#if PHYSICS_THREAD_CONTEXT
	const bool bIsInPhysicsSimContext = IsInPhysicsThreadContext();
	const bool bIsInGameThreadContext = IsInGameThreadContext();
#else
	const bool bIsInPhysicsSimContext = false;
	const bool bIsInGameThreadContext = false;
#endif

#if CHAOS_DEBUG_DRAW
	const ChaosDD::Private::FChaosDDTaskParentContext ParentDDContext;
#else
	int32 ParentDDContext = 0;
#endif

	auto PassThrough = [InCallable, bIsInPhysicsSimContext, bIsInGameThreadContext, &ParentDDContext](int32 Idx)
	{
#if PHYSICS_THREAD_CONTEXT
		FPhysicsThreadContextScope PTScope(bIsInPhysicsSimContext);
		FGameThreadContextScope GTScope(bIsInGameThreadContext);
#endif
#if CHAOS_DEBUG_DRAW
		ChaosDD::Private::FChaosDDScopeTaskContext DDTaskContext(ParentDDContext);
#endif

		InCallable(Idx);
	};

	const bool bSingleThreaded = bSingleWorkerPhysics || bDisablePhysicsParallelFor || bForceSingleThreaded;
	const EParallelForFlags Flags = (bSingleThreaded ? EParallelForFlags::ForceSingleThread : EParallelForFlags::None);
	const int32 MinBatchSize = ((MaxNumWorkers > 0) && (InNum > MaxNumWorkers)) ? FMath::DivideAndRoundUp(InNum, MaxNumWorkers) : 1;

	ParallelFor(TEXT("PhysicsParallelFor"), InNum, MinBatchSize, PassThrough, Flags);
}

void Chaos::PhysicsParallelForRange(int32 InNum, TFunctionRef<void(int32, int32)> InCallable, const int32 InMinBatchSize, bool bForceSingleThreaded)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Chaos_PhysicsParallelFor);
	using namespace Chaos;

	// Passthrough for now, except with global flag to disable parallel
#if PHYSICS_THREAD_CONTEXT
	const bool bIsInPhysicsSimContext = IsInPhysicsThreadContext();
	const bool bIsInGameThreadContext = IsInGameThreadContext();
#else
	const bool bIsInPhysicsSimContext = false;
	const bool bIsInGameThreadContext = false;
#endif

	//calculate the number of workers
	int32 NumWorkers = int32(LowLevelTasks::FScheduler::Get().GetNumWorkers());
	if (!LowLevelTasks::FScheduler::Get().IsWorkerThread())
	{
		NumWorkers++; //named threads help with the work
	}
	NumWorkers = FMath::Min(NumWorkers, InNum);
	NumWorkers = FMath::Min(NumWorkers, MaxNumWorkers);
	check(NumWorkers > 0);
	int32 BatchSize = FMath::DivideAndRoundUp<int32>(InNum, NumWorkers);
	int32 MinBatchSize = FMath::Max(InMinBatchSize, MinRangeBatchSize);
	// @todo(mlentine): Find a better batch size in this case
	if (InNum < MinBatchSize)
	{
		NumWorkers = 1;
		BatchSize = InNum;
	}
	else
	{
		while (BatchSize < MinBatchSize && NumWorkers > 1)
		{
			NumWorkers /= 2;
			BatchSize = FMath::DivideAndRoundUp<int32>(InNum, NumWorkers);
		}
	}
	TArray<int32> RangeIndex;
	RangeIndex.Add(0);
	for (int32 i = 1; i <= NumWorkers; i++)
	{
		int32 PrevEnd = RangeIndex[i - 1];
		int32 NextEnd = FMath::Min(BatchSize + RangeIndex[i - 1], InNum);
		if (NextEnd != PrevEnd)
		{
			RangeIndex.Add(NextEnd);
		}
		else
		{
			break;
		}
	}

	auto PassThrough = [InCallable, &RangeIndex, bIsInPhysicsSimContext, bIsInGameThreadContext](int32 ThreadId)
	{
#if PHYSICS_THREAD_CONTEXT
		FPhysicsThreadContextScope PTScope(bIsInPhysicsSimContext);
		FGameThreadContextScope GTScope(bIsInGameThreadContext);
#endif
		InCallable(RangeIndex[ThreadId], RangeIndex[ThreadId + 1]);
	};
	::ParallelFor(RangeIndex.Num() - 1, PassThrough, bSingleWorkerPhysics || bDisablePhysicsParallelFor || bForceSingleThreaded);
}

void Chaos::PhysicsParallelForWithContext(int32 InNum, TFunctionRef<int32 (int32, int32)> InContextCreator, TFunctionRef<void(int32, int32)> InCallable, bool bForceSingleThreaded)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Chaos_PhysicsParallelFor);
	using namespace Chaos;

	// Passthrough for now, except with global flag to disable parallel
#if PHYSICS_THREAD_CONTEXT
	const bool bIsInPhysicsSimContext = IsInPhysicsThreadContext();
	const bool bIsInGameThreadContext = IsInGameThreadContext();
#else
	const bool bIsInPhysicsSimContext = false;
	const bool bIsInGameThreadContext = false;
#endif

	auto PassThrough = [InCallable, bIsInPhysicsSimContext, bIsInGameThreadContext](int32 ContextIndex, int32 Idx)
	{
#if PHYSICS_THREAD_CONTEXT
		FPhysicsThreadContextScope PTScope(bIsInPhysicsSimContext);
		FGameThreadContextScope GTScope(bIsInGameThreadContext);
#endif
		InCallable(Idx, ContextIndex);
	};

	const bool bSingleThreaded = bSingleWorkerPhysics || bDisablePhysicsParallelFor || bForceSingleThreaded;
	const EParallelForFlags Flags = bSingleThreaded ? (EParallelForFlags::ForceSingleThread) : (EParallelForFlags::None);
	const int32 MinBatchSize = ((MaxNumWorkers > 0) && (InNum > MaxNumWorkers)) ? FMath::DivideAndRoundUp(InNum, MaxNumWorkers) : 1;

	// Unfortunately ParallelForWithTaskContext takes an array of context objects - we don't use it and in our case
	// it ends up being an array where array[index] = index.
	// The reason we don't need it is that our ContextCreator returns the context index we want to use on a given
	// worker thread, and this is passed to the user function. The user function can just captures its array of
	// contexts and use the context indeex to get its context from it.
	TArray<int32, TInlineAllocator<16>> Contexts;

	::ParallelForWithTaskContext(TEXT("PhysicsParallelForWithContext"), Contexts, InNum, MinBatchSize, InContextCreator, PassThrough, Flags);
}

bool Chaos::ShouldExecuteTasks(int32 InInstanceNum)
{
	if (bSingleWorkerPhysics)
	{
		// Single threaded physics should not run in parallel
		return false;
	}
	else if (InInstanceNum >= GlobalSmallBatchSize)
	{
		// Meets threshold so run in parallel
		return true;
	}
	return false;
}


