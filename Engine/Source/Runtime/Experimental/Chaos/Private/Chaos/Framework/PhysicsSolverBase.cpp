// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/Framework/PhysicsSolverBase.h"
#include "Chaos/Framework/PhysicsProxyBase.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "ChaosStats.h"
#include "Chaos/PendingSpatialData.h"
#include "PBDRigidsSolver.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "Chaos/Framework/ChaosResultsManager.h"
#include "ChaosSolversModule.h"
#include "Framework/Threading.h"
#include "RewindData.h"

#include "ChaosVisualDebugger/ChaosVisualDebuggerTrace.h"
#include "ChaosVisualDebugger/ChaosVDContextProvider.h"
#include "HAL/LowLevelMemTracker.h"

DEFINE_STAT(STAT_AsyncPullResults);
DEFINE_STAT(STAT_AsyncInterpolateResults);
DEFINE_STAT(STAT_SyncPullResults);
DEFINE_STAT(STAT_ProcessSingleProxy);
DEFINE_STAT(STAT_ProcessGCProxy);
DEFINE_STAT(STAT_ProcessClusterUnionProxy);
DEFINE_STAT(STAT_PullConstraints);

CSV_DEFINE_CATEGORY(ChaosPhysicsSolver, true);

namespace Chaos
{	
	CHAOS_API int32 RewindBeforeAdvance = 1;
	FAutoConsoleVariableRef CVarReorderRewindResimulation(TEXT("p.Resim.RewindBeforeAdvance"), RewindBeforeAdvance, TEXT("Whether to apply the rewind before or after the advance"));

	CHAOS_API int32 ResimulateOnSpecifiedStep = 1;
	FAutoConsoleVariableRef CVarResimulateOnSpecifiedStep(TEXT("p.Resim.ResimulateOnSpecifiedStep"), ResimulateOnSpecifiedStep
		, TEXT("0 = Resimulation can trigger on any physics step. \n -1 or other negative values = Resimulation only happens on the last physics step (if multiple are queued up at the same time). \n N = Resimulation only happens on the specified step or last, whichever comes first (if multiple are queued up at the same time)"));

	/** Check if we can enable debugging informations for network physics */
	static int32 DebugNetworkPhysicsPrediction = 0;
	static FAutoConsoleVariableRef CVarDebugNetworkPhysicsPrediction(TEXT("np2.DebugNetworkPhysicsPrediction"), DebugNetworkPhysicsPrediction, TEXT("Debugs network physics prediction"));
	/*static*/ bool FPhysicsSolverBase::CanDebugNetworkPhysicsPrediction()
	{
		return !!DebugNetworkPhysicsPrediction;
	}

	/** Return the interpolation lerp in case the resim is off */
	static float NetworkPhysicsPredictionInterpLerp = 0.1f;
	static FAutoConsoleVariableRef CVarNetworkPhysicsPredictionInterpLerp(TEXT("np2.NetworkPhysicsPredictionInterpLerp"), NetworkPhysicsPredictionInterpLerp, TEXT("State lerp value in between the target state and the current one in case resim is disabled or if the pawn is not possessed (continuous correction)"));
	/*static*/ float FPhysicsSolverBase::NetworkPhysicsInterpolationLerp()
	{
		return NetworkPhysicsPredictionInterpLerp;
	}

	extern int GSingleThreadedPhysics;
	void FPhysicsSolverBase::ChangeBufferMode(EMultiBufferMode InBufferMode)
	{
		BufferMode = InBufferMode;
	}

	FDelegateHandle FPhysicsSolverEvents::AddPreAdvanceCallback(FSolverPreAdvance::FDelegate InDelegate)
	{
		return EventPreSolve.Add(InDelegate);
	}

	bool FPhysicsSolverEvents::RemovePreAdvanceCallback(FDelegateHandle InHandle)
	{
		return EventPreSolve.Remove(InHandle);
	}

	FDelegateHandle FPhysicsSolverEvents::AddPreBufferCallback(FSolverPreBuffer::FDelegate InDelegate)
	{
		return EventPreBuffer.Add(InDelegate);
	}

	bool FPhysicsSolverEvents::RemovePreBufferCallback(FDelegateHandle InHandle)
	{
		return EventPreBuffer.Remove(InHandle);
	}

	FDelegateHandle FPhysicsSolverEvents::AddPostAdvanceCallback(FSolverPostAdvance::FDelegate InDelegate)
	{
		return EventPostSolve.Add(InDelegate);
	}

	bool FPhysicsSolverEvents::RemovePostAdvanceCallback(FDelegateHandle InHandle)
	{
		return EventPostSolve.Remove(InHandle);
	}

	FDelegateHandle FPhysicsSolverEvents::AddTeardownCallback(FSolverTeardown::FDelegate InDelegate)
	{
		return EventTeardown.Add(InDelegate);
	}

	bool FPhysicsSolverEvents::RemoveTeardownCallback(FDelegateHandle InHandle)
	{
		return EventTeardown.Remove(InHandle);
	}

	FAutoConsoleTaskPriority CPrio_FPhysicsTickTask(
		TEXT("TaskGraph.TaskPriorities.PhysicsTickTask"),
		TEXT("Task and thread priotiry for Chaos physics tick"),
		ENamedThreads::HighThreadPriority, // if we have high priority task threads, then use them...
		ENamedThreads::NormalTaskPriority, // .. at normal task priority
		ENamedThreads::HighTaskPriority // if we don't have hi pri threads, then use normal priority threads at high task priority instead
	);

	int32 PhysicsRunsOnGT = 0;
	FAutoConsoleVariableRef CVarPhysicsRunsOnGT(TEXT("p.PhysicsRunsOnGT"), PhysicsRunsOnGT, TEXT("If true the physics thread runs on the game thread, but will still go wide on tasks like collision detection"));

	ENamedThreads::Type FPhysicsSolverProcessPushDataTask::GetDesiredThread()
	{
		return CPrio_FPhysicsTickTask.Get();
	}

	ENamedThreads::Type FPhysicsSolverAdvanceTask::GetDesiredThread()
	{
		return PhysicsRunsOnGT == 0 ? CPrio_FPhysicsTickTask.Get() : ENamedThreads::GameThread;
	}

	ENamedThreads::Type FPhysicsSolverRewindTask::GetDesiredThread()
	{
		return PhysicsRunsOnGT == 0 ? CPrio_FPhysicsTickTask.Get() : ENamedThreads::GameThread;
	}

	void FPhysicsSolverProcessPushDataTask::ProcessPushData()
	{
		using namespace Chaos;

		LLM_SCOPE(ELLMTag::ChaosUpdate);
		SCOPE_CYCLE_COUNTER(STAT_ChaosTick);
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Physics);
		CSV_SCOPED_TIMING_STAT(PhysicsVerbose, StepSolver_PushData);
		
		CVD_SCOPE_CONTEXT(Solver.GetChaosVDContextData());

#if PHYSICS_THREAD_CONTEXT
		FPhysicsThreadContextScope Scope(/*IsPhysicsThreadContext=*/true);
#endif

		Solver.SetExternalTimestampConsumed_Internal(PushData->ExternalTimestamp);
		Solver.ProcessPushedData_Internal(*PushData);
		
		Solver.PrepareAdvanceBy(PushData->ExternalDt);

	}

	void FPhysicsSolverFrozenGTPreSimCallbacks::GTPreSimCallbacks()
	{
		LLM_SCOPE(ELLMTag::ChaosUpdate);
		SCOPE_CYCLE_COUNTER(STAT_ChaosTick);
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Physics);
		CSV_SCOPED_TIMING_STAT(PhysicsVerbose, StepSolver_GTPreSimCallbacks);

		//We are on GT, but we know PhysicsThread is waiting so we're actually going to operate on PT data
#if PHYSICS_THREAD_CONTEXT
		FPhysicsThreadContextScope Scope(/*IsPhysicsThreadContext=*/true);
		FFrozenGameThreadContextScope FrozenScope;	//Make sure we fire ensures if any physics GT data is used
#endif

		Solver.SetGameThreadFrozen(true);
		Solver.ApplyCallbacks_Internal();
		Solver.SetGameThreadFrozen(false);
		
	}

	void FPhysicsSolverRewindTask::RewindSolver()
	{
		LLM_SCOPE(ELLMTag::ChaosUpdate);
		SCOPE_CYCLE_COUNTER(STAT_ChaosTick);
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Physics);
		CSV_SCOPED_TIMING_STAT(PhysicsVerbose, StepSolver_RewindData);

		CVD_SCOPE_CONTEXT(Solver.GetChaosVDContextData());

#if PHYSICS_THREAD_CONTEXT
		FPhysicsThreadContextScope Scope(/*IsPhysicsThreadContext=*/true);
#endif

		if (PushData && !Solver.IsResimming())
		{
			if (RewindBeforeAdvance != 0)
			{
				Solver.AddResimulationRequest_Internal(*PushData);
			}

			// Check if a resimulation is allowed based on which step to resim on when multiple physics steps are scheduled in a row from the game thread
			const int32 SpecifiedStep = ResimulateOnSpecifiedStep < 0
				? PushData->IntervalNumSteps // Only resimulate on the last frame
				: FMath::Min(ResimulateOnSpecifiedStep, PushData->IntervalNumSteps); // Resimulate on specified step or last if not enough steps

			const bool bAllowResimulation = ResimulateOnSpecifiedStep == 0 
				? true // Always allow resimulation
				: (PushData->IntervalStep + 1) == SpecifiedStep; // Allow resimulation on specific step

			if (bAllowResimulation)
			{
				SCOPE_CYCLE_COUNTER(STAT_ConditionalApplyRewind);
				Solver.ConditionalApplyRewind_Internal();
			}
		}
	}

	FPhysicsSolverAdvanceTask::FPhysicsSolverAdvanceTask(FPhysicsSolverBase& InSolver, FPushPhysicsData* InPushData)
		: Solver(InSolver)
		, PushData(InPushData)
	{
		CVD_GET_CURRENT_CONTEXT(CVDContext);
		Solver.NumPendingSolverAdvanceTasks++;
	}

	void FPhysicsSolverAdvanceTask::DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		CVD_SCOPE_CONTEXT(CVDContext);

		AdvanceSolver();
	}


	void FPhysicsSolverAdvanceTask::AdvanceSolver()
	{
#if WITH_CHAOS_VISUAL_DEBUGGER
		// TOptional lets us conditionally construct the scoped CVD frame tracer while
		// keeping it alive for the entire AdvanceSolver() scope. During resimulation the CVD
		// frame is managed by ConditionalApplyRewind_Internal instead, so that all
		// resim stages (RewindToFrame, StepNonResimParticles, ApplyTargets, solver
		// evolution, etc.) appear as stages within a single frame per physics step.
		TOptional<FChaosVDScopeSolverFrame<FPhysicsSolverBase>> ScopeSolverFrame;
		if (!Solver.IsResimming())
		{
			ScopeSolverFrame.Emplace(Solver);
		}
#endif
		LLM_SCOPE(ELLMTag::ChaosUpdate);
		SCOPE_CYCLE_COUNTER(STAT_ChaosTick);
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Physics);
		CSV_SCOPED_TIMING_STAT(PhysicsVerbose, StepSolver);
		PHYSICS_CSV_CUSTOM_EXPENSIVE(PhysicsCounters, NumPendingSolverAdvanceTasks, NumPendingSolverAdvanceTasks, ECsvCustomStatOp::Max);

#if PHYSICS_THREAD_CONTEXT
		FPhysicsThreadContextScope Scope(/*IsPhysicsThreadContext=*/true);
#endif

#if UE_WITH_REMOTE_OBJECT_HANDLE
		FPhysicsSceneGuardScopedWrite ScopedWrite(Solver.GetInternalDataLock());
#endif

		// StepFraction: how much of the remaining time this step represents while substepping, used to interpolate kinematic targets
		// E.g., for 4 steps this will be: 1/4, 1/3, 1/2, 1
		const FReal PseudoFraction = PushData->bSolverSubstepped ? (FReal)1 / (FReal)(PushData->IntervalNumSteps - PushData->IntervalStep) : 1.0f;
		const FSubStepInfo SubStepInfo = { PseudoFraction, PushData->IntervalStep, PushData->IntervalNumSteps, PushData->bSolverSubstepped };

		Solver.AdvanceSolverBy(SubStepInfo);
		{
			SCOPE_CYCLE_COUNTER(STAT_ResetMarshallingData);
			Solver.GetMarshallingManager().FreeDataToHistory_Internal(PushData);	//cannot use push data after this point
			PushData = nullptr;
		}
		Solver.NumPendingSolverAdvanceTasks--;
	}

	CHAOS_API int32 UseAsyncInterpolation = 1;
	FAutoConsoleVariableRef CVarUseAsyncInterpolation(TEXT("p.UseAsyncInterpolation"), UseAsyncInterpolation, TEXT("Whether to interpolate when async mode is enabled"));

	CHAOS_API int32 ForceDisableAsyncPhysics = 0;
	FAutoConsoleVariableRef CVarForceDisableAsyncPhysics(TEXT("p.ForceDisableAsyncPhysics"), ForceDisableAsyncPhysics, TEXT("Whether to force async physics off regardless of other settings"));

	auto LambdaMul = FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
		{
			for (FPhysicsSolverBase* Solver : FChaosSolversModule::GetModule()->GetAllSolvers())
			{
				Solver->SetAsyncInterpolationMultiplier(InVariable->GetFloat());
			}
		});

	auto LambdaAsyncMode = FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
		{
			for (FPhysicsSolverBase* Solver : FChaosSolversModule::GetModule()->GetAllSolvers())
			{
				Solver->SetAsyncPhysicsBlockMode(EAsyncBlockMode(InVariable->GetInt()));
			}
		});

	CHAOS_API FRealSingle AsyncInterpolationMultiplier = 2.f;
	FAutoConsoleVariableRef CVarAsyncInterpolationMultiplier(TEXT("p.AsyncInterpolationMultiplier"), AsyncInterpolationMultiplier, TEXT("How many multiples of the fixed dt should we look behind for interpolation"), LambdaMul);

	// 0 blocks on any physics steps generated from past GT Frames, and blocks on none of the tasks from current frame.
	// 1 blocks on everything except the single most recent task (including tasks from current frame)
	// 1 should guarantee we will always have a future output for interpolation from 2 frames in the past
	// 2 doesn't block the game thread. Physics steps could be eventually be dropped if taking too much time.
	int32 AsyncPhysicsBlockMode = 0;
	FAutoConsoleVariableRef CVarAsyncPhysicsBlockMode(TEXT("p.AsyncPhysicsBlockMode"), AsyncPhysicsBlockMode, TEXT("Setting to 0 blocks on any physics steps generated from past GT Frames, and blocks on none of the tasks from current frame."
		" 1 blocks on everything except the single most recent task (including tasks from current frame). 1 should gurantee we will always have a future output for interpolation from 2 frames in the past."
		" 2 doesn't block the game thread, physics steps could be eventually be dropped if taking too much time."), LambdaAsyncMode);

	FPhysicsSolverBase::FPhysicsSolverBase(const EMultiBufferMode BufferingModeIn,const EThreadingModeTemp InThreadingMode,UObject* InOwner, Chaos::FReal InAsyncDt)
		: BufferMode(BufferingModeIn)
		, ThreadingMode(!!GSingleThreadedPhysics ? EThreadingModeTemp::SingleThread : InThreadingMode)
#if CHAOS_DEBUG_NAME
		, DebugName(NAME_None)
#endif
		, PullResultsManager(MakeUnique<FChaosResultsManager>(MarshallingManager))
		, PendingSpatialOperations_External(MakeUnique<FPendingSpatialDataQueue>())
		, bUseCollisionResimCache(false)
		, NumPendingSolverAdvanceTasks(0)
		, bPaused_External(false)
		, Owner(InOwner)
		, ExternalDataLock_External(new FPhysSceneLock())
#if UE_WITH_REMOTE_OBJECT_HANDLE
		, InternalDataLock(new FPhysSceneLock())
#endif
		, bIsShuttingDown(false)
		, AsyncDt(InAsyncDt)
		, AccumulatedTime(0)
		, MMaxDeltaTime(0.0)
		, MMinDeltaTime(UE_SMALL_NUMBER)
		, MMaxSubSteps(1)
		, ExternalSteps(0)
		, AsyncBlockMode(EAsyncBlockMode(AsyncPhysicsBlockMode))
		, AsyncMultiplier(AsyncInterpolationMultiplier)
#if !UE_BUILD_SHIPPING
		, bStealAdvanceTasksForTesting(false)
#endif
	{
		UE_LOGF(LogChaos, Verbose, "FPhysicsSolverBase::AsyncDt:%f", IsUsingAsyncResults() ? AsyncDt : -1);

		//If user is running with -PhysicsRunsOnGT override the cvar (doing it here to avoid parsing every time task is scheduled)
		if(FParse::Param(FCommandLine::Get(), TEXT("PhysicsRunsOnGT")))
		{
			PhysicsRunsOnGT = 1;
		}
	}

#if CHAOS_SOLVER_DEBUG_NAME
	void FPhysicsSolverBase::SetDebugName(const FName& Name)
	{
		DebugName = Name;
		OnDebugNameChanged();
	}
#endif

	FName FPhysicsSolverBase::GetDebugName() const
	{
#if CHAOS_SOLVER_DEBUG_NAME
		return DebugName;
#else
		return NAME_None;
#endif
	}

	void FPhysicsSolverBase::EnableAsyncMode(FReal FixedDt)
	{
		AsyncDt = FixedDt;
		UE_LOGF(LogChaos, Verbose, "FPhysicsSolverBase::AsyncDt:%f", IsUsingAsyncResults() ? AsyncDt : -1);
	}

	void FPhysicsSolverBase::DisableAsyncMode()
	{
		AsyncDt = -1;
		UE_LOGF(LogChaos, Verbose, "FPhysicsSolverBase::AsyncDt:%f", AsyncDt);
	}

	FPhysicsSolverBase::~FPhysicsSolverBase()
	{
		//reset history buffer before freeing any unremoved callback objects
		MarshallingManager.SetHistoryLength_Internal(0);

		//if any callback objects are still registered, just delete them here
		for(ISimCallbackObject* CallbackObject : SimCallbackObjects)
		{
			delete CallbackObject;
		}
	}

	void FPhysicsSolverBase::DestroySolver(FPhysicsSolverBase& InSolver)
	{
		// Please read the comments this is a minefield.
				
		const bool bIsSingleThreadEnvironment = FPlatformProcess::SupportsMultithreading() == false;
		if (bIsSingleThreadEnvironment == false)
		{
			// In Multithreaded: DestroySolver should only be called if we are not waiting on async work.
			// This should be called when World/Scene are cleaning up, World implements IsReadyForFinishDestroy() and returns false when async work is still going.
			// This means that garbage collection should not cleanup world and this solver until this async work is complete.
			// We do it this way because it is unsafe for us to block on async task in this function, as it is unsafe to block on a task during GC, as this may schedule
			// another task that may be unsafe during GC, and cause crashes.
			ensure(InSolver.IsPendingTasksComplete());
		}
		else
		{
			// In Singlethreaded: We cannot wait for any tasks in IsReadyForFinishDestroy() (on World) so it always returns true in single threaded.
			// Task will never complete during GC in single theading, as there are no threads to do it.
			// so we have this wait below to allow single threaded to complete pending tasks before solver destroy.

			InSolver.WaitOnPendingTasks_External();
		}

		// GeometryCollection particles do not always remove collision constraints on unregister,
		// explicitly clear constraints so we will not crash when filling collision events in advance.
		// @todo(chaos): fix this and remove
		{
			auto* Evolution = static_cast<FPBDRigidsSolver&>(InSolver).GetEvolution();
			if (Evolution)
			{
				Evolution->ResetConstraints();
			}
		}

		// Advance in single threaded because we cannot block on an async task here if in multi threaded mode. see above comments.
		InSolver.SetThreadingMode_External(EThreadingModeTemp::SingleThread);
		InSolver.MarkShuttingDown();
		{
			InSolver.AdvanceAndDispatch_External(0);	//flush any pending commands are executed (for example unregister object)
		}

		// verify callbacks have been processed and we're not leaking.
		// TODO: why is this still firing in 14.30? (Seems we're still leaking)
		//ensure(InSolver.SimCallbacks.Num() == 0);

		delete &InSolver;
	}
	
	void FPhysicsSolverBase::AddResimulationRequest_Internal(FPushPhysicsData& PushData)
	{
		// Add resimulation request from rewind callbacks inputs
		if (!IsShuttingDown() && !bGameThreadFrozen && MRewindCallback)
		{
			// Set all the input on the matching callbacks so they are available for the resimulation request below.
			// Note: ProcessPushData_Internal also calls SetCurrentInput_Internal for the general sim callback flow.
			for (FSimCallbackInputAndObject& InputAndCallbackObj : PushData.SimCallbackInputs)
			{
				if(InputAndCallbackObj.CallbackObject)
				{
					InputAndCallbackObj.CallbackObject->SetCurrentInput_Internal(InputAndCallbackObj.Input);
				}
			}

			QUICK_SCOPE_CYCLE_COUNTER(Chaos_RewindCallback_ProcessInputs_Internal);
			MRewindCallback->AddResimulationRequest_Internal(PushData.InternalStep, (float)MLastDt);
		}
	}

	void FPhysicsSolverBase::ApplyCallbacks_Internal()
	{
		QUICK_SCOPE_CYCLE_COUNTER(ApplySimCallbacks);

		//if delta time is 0 we are flushing data, user callbacks should not be triggered because there is no sim
		if (MLastDt > 0)
		{
			const FReal SimTime = GetSolverTime();

			for (ISimCallbackObject* Callback : SimCallbackObjects)
			{
				if (Callback->HasOption(ESimCallbackOptions::RunOnFrozenGameThread) == bGameThreadFrozen)
				{
					Callback->SetSimAndDeltaTime_Internal(SimTime, MLastDt);
				}
			}

			if (FPushPhysicsData* PushData = MarshallingManager.GetConsumerData_Internal())
			{
				if (MRewindCallback && !IsShuttingDown() && !bGameThreadFrozen)
				{
					QUICK_SCOPE_CYCLE_COUNTER(Chaos_RewindCallback_PreProcessInputs_Internal);
					MRewindCallback->PreProcessInputs_Internal(PushData->InternalStep);
				}

				for (ISimCallbackObject* Callback : ProcessInputsWatchers)
				{
					if (Callback->HasOption(ESimCallbackOptions::RunOnFrozenGameThread) == bGameThreadFrozen)
					{
						FScopedTraceSolverCallback TraceCallback(Callback);

						Callback->ProcessInputs_Internal(PushData->InternalStep);
					}
				}

				if (MRewindCallback && !IsShuttingDown() && !bGameThreadFrozen)
				{
					{
						QUICK_SCOPE_CYCLE_COUNTER(Chaos_RewindCallback_ProcessInputs_Internal);
						MRewindCallback->ProcessInputs_Internal(PushData->InternalStep, PushData->SimCallbackInputs);
					}
					{
						QUICK_SCOPE_CYCLE_COUNTER(Chaos_RewindCallback_PostProcessInputs_Internal);
						MRewindCallback->PostProcessInputs_Internal(PushData->InternalStep);
					}
				}
			}

			for (ISimCallbackObject* Callback : SimCallbackObjects)
			{
				if (Callback->HasOption(ESimCallbackOptions::RunOnFrozenGameThread) == bGameThreadFrozen)
				{
					FScopedTraceSolverCallback TraceCallback(Callback);

					Callback->PreSimulate_Internal();
				}
			}
		}
	}

	void FPhysicsSolverBase::UpdateParticleInAccelerationStructure_External(FGeometryParticle* Particle, EPendingSpatialDataOperation InOperation)
	{
		//mark it as pending for async structure being built
		FAccelerationStructureHandle AccelerationHandle(Particle);
		FPendingSpatialData& SpatialData = PendingSpatialOperations_External->FindOrAdd(Particle->UniqueIdx());

		//make sure any new operations (i.e not currently being consumed by sim) are not acting on a deleted object
		ensure(SpatialData.SyncTimestamp < MarshallingManager.GetExternalTimestamp_External() || SpatialData.Operation != EPendingSpatialDataOperation::Delete);

		SpatialData.Operation = InOperation;
		SpatialData.SpatialIdx = Particle->SpatialIdx();
		SpatialData.AccelerationHandle = AccelerationHandle;
		SpatialData.SyncTimestamp = MarshallingManager.GetExternalTimestamp_External();
	}

	void FPhysicsSolverBase::EnqueueSimcallbackRewindRegisteration(ISimCallbackObject* Callback)
	{
		EnqueueCommandImmediate([this, Callback]()
		{
			if (ensure(MRewindCallback.IsValid()))
			{
				MRewindCallback->RegisterRewindableSimCallback_Internal(Callback);
			}
		});
	}

#if !UE_BUILD_SHIPPING
	void FPhysicsSolverBase::SetStealAdvanceTasks_ForTesting(bool bInStealAdvanceTasksForTesting)
	{
		bStealAdvanceTasksForTesting = bInStealAdvanceTasksForTesting;
	}

	void FPhysicsSolverBase::PopAndExecuteStolenAdvanceTask_ForTesting()
	{
		ensure(ThreadingMode == EThreadingModeTemp::SingleThread);
		if (ensure(StolenSolverAdvanceTasks.Num() > 0))
		{
			StolenSolverAdvanceTasks[0].AdvanceSolver();
			StolenSolverAdvanceTasks.RemoveAt(0);
		}
	}
#endif

	void FPhysicsSolverBase::TrackGTParticle_External(FGeometryParticle& Particle)
	{
		const int32 Idx = Particle.UniqueIdx().Idx;
		const int32 SlotsNeeded = Idx + 1 - UniqueIdxToGTParticles.Num();
		if (SlotsNeeded > 0)
		{
			UniqueIdxToGTParticles.AddZeroed(SlotsNeeded);
		}

		UniqueIdxToGTParticles[Idx] = &Particle;
	}

	void FPhysicsSolverBase::ClearGTParticle_External(FGeometryParticle& Particle)
	{
		const int32 Idx = Particle.UniqueIdx().Idx;
		if (ensure(Idx < UniqueIdxToGTParticles.Num()))
		{
			UniqueIdxToGTParticles[Idx] = nullptr;
		}
	}

	void FPhysicsSolverBase::SetRewindCallback(TUniquePtr<IRewindCallback>&& RewindCallback)
	{
		ensure(RewindCallback);
		MRewindCallback = MoveTemp(RewindCallback);

		if (MRewindData.IsValid())
		{
			MRewindCallback->RewindData = MRewindData.Get();
		}
	}

	int32 MaxPhysicsStepsPerGameTick = 3;
	FAutoConsoleVariableRef CVarMaxPhysicsStepsPerGameTick(TEXT("p.MaxPhysicsStepsPerGameTick"), MaxPhysicsStepsPerGameTick, TEXT("The maximum number of physics steps per gametick frame."));

	FGraphEventRef FPhysicsSolverBase::AdvanceAndDispatch_External(FReal InDt)
	{
		LLM_SCOPE(ELLMTag::ChaosScene);
		const bool bSubstepping = MMaxSubSteps > 1;
		SetSolverSubstep_External(bSubstepping);
		const FReal DtWithPause = bPaused_External ? 0.0f : InDt;
		FReal InternalDt = DtWithPause;
		int32 NumSteps = 1;

		if(IsUsingFixedDt())
		{
			AccumulatedTime += DtWithPause;
			if(InDt == 0)	//this is a special flush case
			{
				//just use any remaining time and sync up to latest no matter what
				InternalDt = AccumulatedTime;
				NumSteps = 1;
				AccumulatedTime = 0;
			}
			else
			{
				InternalDt = AsyncDt;
				NumSteps = FMath::FloorToInt32(AccumulatedTime / InternalDt);
				AccumulatedTime -= InternalDt * static_cast<FReal>(NumSteps);
			}
		}
		else if (bSubstepping && InDt > 0)
		{
			NumSteps = FMath::CeilToInt32(DtWithPause / MMaxDeltaTime);
			if (NumSteps > MMaxSubSteps)
			{
				// Hitting this case means we're losing time, given the constraints of MaxSteps and MaxDt we can't
				// fully handle the Dt requested, the simulation will appear to the viewer to run slower than realtime
				NumSteps = MMaxSubSteps;
				InternalDt = MMaxDeltaTime;
			}
			else
			{
				InternalDt = DtWithPause / static_cast<FReal>(NumSteps);
			}
		}

		if(InDt > 0)
		{
			ExternalSteps++;	//we use this to average forces. It assumes external dt is about the same. 0 dt should be ignored as it typically has nothing to do with force
		}

		// Eventually drop physics steps in mode 2
		if (AsyncBlockMode == EAsyncBlockMode::DoNoBlock)
		{
			// Make sure not to accumulate too many physics solver tasks.
			const int32 MaxPhysicsStepToKeep = MaxPhysicsStepsPerGameTick;
			const int32 MaxNumSteps = MaxPhysicsStepToKeep - NumPendingSolverAdvanceTasks;
			if (NumSteps > MaxNumSteps)
			{
				CSV_CUSTOM_STAT(ChaosPhysicsSolver, PhysicsFrameDropped, NumSteps - MaxNumSteps, ECsvCustomStatOp::Accumulate);
				// NumSteps + NumPendingSolverAdvanceTasks shouldn't be bigger than MaxPhysicsStepToKeep
				NumSteps = FMath::Min<int32>(NumSteps, MaxNumSteps);
			}
		}
			
		if (NumSteps > 0)
		{
			//make sure any GT state is pushed into necessary buffer
			PushPhysicsState(InternalDt, NumSteps, FMath::Max(ExternalSteps, 1));
			ExternalSteps = 0;
		}

		// If standalone solver we are not responsible to spawn tasks
		if(IsStandaloneSolver())
		{
			return {};
		}

		// Ensures we block on any tasks generated from previous frames
		FGraphEventRef BlockingTasks = PendingTasks;

		while(FPushPhysicsData* PushData = MarshallingManager.StepInternalTime_External())
		{
			if (!bIsShuttingDown)
			{
				for (FSimCallbackInputAndObject& Callback : PushData->SimCallbackInputs)
				{
					if (Callback.CallbackObject && Callback.CallbackObject->HasOption(ESimCallbackOptions::ProcessInputsExternal))
					{
						FScopedTraceSolverCallback TraceCallback(Callback.CallbackObject);

						Callback.CallbackObject->ProcessInputs_External(PushData->InternalStep);
					}
				}

				if (MRewindCallback)
				{
					MRewindCallback->ProcessInputs_External(PushData->InternalStep, PushData->SimCallbackInputs);
				}
			}

			if(ThreadingMode == EThreadingModeTemp::SingleThread)
			{
				ensure(!PendingTasks || PendingTasks->IsComplete());	//if mode changed we should have already blocked
				FAllSolverTasks ImmediateTask(*this, PushData);
#if !UE_BUILD_SHIPPING
				if(bStealAdvanceTasksForTesting)
				{
					StolenSolverAdvanceTasks.Emplace(MoveTemp(ImmediateTask));
				}
				else
				{
					ImmediateTask.AdvanceSolver();
				}
#else
				ImmediateTask.AdvanceSolver();
#endif
			}
			else
			{
				// If enabled, block on all but most recent physics task, even tasks generated this frame.
				if (AsyncBlockMode == EAsyncBlockMode::BlockForBestInterpolation)
				{
					BlockingTasks = PendingTasks;
				}

				FGraphEventArray Prereqs;
				if (PendingTasks && !PendingTasks->IsComplete())
				{
					Prereqs.Add(PendingTasks);
				}

				if(RewindBeforeAdvance == 0)
				{ 
					PendingTasks = TGraphTask<FPhysicsSolverProcessPushDataTask>::CreateTask(&Prereqs).ConstructAndDispatchWhenReady(*this, PushData);
					Prereqs.Add(PendingTasks);

					if (bSolverHasFrozenGameThreadCallbacks)
					{
						PendingTasks = TGraphTask<FPhysicsSolverFrozenGTPreSimCallbacks>::CreateTask(&Prereqs).ConstructAndDispatchWhenReady(*this);
						Prereqs.Add(PendingTasks);
					}

					PendingTasks = TGraphTask<FPhysicsSolverAdvanceTask>::CreateTask(&Prereqs).ConstructAndDispatchWhenReady(*this, PushData);
					Prereqs.Add(PendingTasks);

					PendingTasks = TGraphTask<FPhysicsSolverRewindTask>::CreateTask(&Prereqs).ConstructAndDispatchWhenReady(*this, PushData);
				}
				else
				{
					PendingTasks = TGraphTask<FPhysicsSolverRewindTask>::CreateTask(&Prereqs).ConstructAndDispatchWhenReady(*this, PushData);
					Prereqs.Add(PendingTasks);

					PendingTasks = TGraphTask<FPhysicsSolverProcessPushDataTask>::CreateTask(&Prereqs).ConstructAndDispatchWhenReady(*this, PushData);
					Prereqs.Add(PendingTasks);

					if (bSolverHasFrozenGameThreadCallbacks)
					{
						PendingTasks = TGraphTask<FPhysicsSolverFrozenGTPreSimCallbacks>::CreateTask(&Prereqs).ConstructAndDispatchWhenReady(*this);
						Prereqs.Add(PendingTasks);
					}

					PendingTasks = TGraphTask<FPhysicsSolverAdvanceTask>::CreateTask(&Prereqs).ConstructAndDispatchWhenReady(*this, PushData);
				}

				if (IsUsingAsyncResults() == false)
				{
					BlockingTasks = PendingTasks;	//block right away
				}
			}

			// This break is mainly here to satisfy unit testing. The call to StepInternalTime_External will decrement the
			// delay in the marshaling manager and throw of tests that are explicitly testing for propagation delays
			if (IsUsingAsyncResults() == false && !bSubstepping)
			{
				break;
			}
		}
		if (AsyncBlockMode == EAsyncBlockMode::DoNoBlock)
		{
			return {};
		}
		return BlockingTasks;
	}


	void FAllSolverTasks::AdvanceSolver()
	{
		if (RewindBeforeAdvance == 0)
		{
			ProcessPushData.ProcessPushData();
			GTPreSimCallbacks.GTPreSimCallbacks();
			AdvanceTask.AdvanceSolver();
			RewindTask.RewindSolver();
		}
		else
		{
			RewindTask.RewindSolver();
			ProcessPushData.ProcessPushData();
			GTPreSimCallbacks.GTPreSimCallbacks();
			AdvanceTask.AdvanceSolver();
		}
	}

	void FSolverTasksPTOnly::AdvanceSolver()
	{
		if (RewindBeforeAdvance == 0)
		{
			ProcessPushData.ProcessPushData();
			AdvanceTask.AdvanceSolver();
			RewindTask.RewindSolver();
		}
		else
		{
			RewindTask.RewindSolver();
			ProcessPushData.ProcessPushData();
			AdvanceTask.AdvanceSolver();
		}
	}
}
