// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/GeometryParticles.h"

#include "Chaos/CastingUtilities.h"
#include "Chaos/ImplicitObject.h"
#include "Chaos/ImplicitObjectUnion.h"
#include "Chaos/Particle/ParticleUtilities.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/Framework/ArrayAlgorithm.h"
#include "Chaos/Framework/PhysicsSolverBase.h"

namespace Chaos
{
	extern void UpdateShapesArrayFromGeometry(FShapeInstanceArray& ShapesArray, const FImplicitObjectPtr& Geometry, const FRigidTransform3& ActorTM);

	namespace Private
	{
		// The name shown for particoles that have not had their DebugName set
		FString EmptyParticleName = "<NoName>";
	}

	FShapeOrShapesArray::FShapeOrShapesArray(const FGeometryParticleHandle* Particle)
	{
		if (Particle)
		{
			const FImplicitObjectRef Geometry = Particle->GetGeometry();
			if (Geometry)
			{
				if (Geometry->IsUnderlyingUnion())
				{
					ShapeArray = &Particle->ShapesArray();
					bIsSingleShape = false;
				}
				else
				{
					Shape = Particle->ShapesArray()[0].Get();
					bIsSingleShape = true;
				}

				return;
			}
		}

		Shape = nullptr;
		bIsSingleShape = true;
	}

	template <typename T, int d, EGeometryParticlesSimType SimType>
	void TGeometryParticlesImp<T, d, SimType>::RemoveShapesAtSortedIndices(const int32 ParticleIndex, const TArrayView<const int32>& InIndices)
	{
		RemoveArrayItemsAtSortedIndices(MShapesArray[ParticleIndex], InIndices);

		// ShapeIdx need to be be updated 
		const int32 NumShapes = MShapesArray[ParticleIndex].Num();
		for (int32 ShapeIndex = 0; ShapeIndex < NumShapes; ++ShapeIndex)
		{
			const FShapeInstancePtr& Shape = MShapesArray[ParticleIndex][ShapeIndex];
			if (Shape)
			{
				Shape->ModifyShapeIndex(ShapeIndex);
			}
		}
	}

	template <typename T, int d, EGeometryParticlesSimType SimType>
	void TGeometryParticlesImp<T, d, SimType>::UpdateShapesArray(const int32 Index)
	{
		UpdateShapesArrayFromGeometry(MShapesArray[Index], GetGeometry(Index), FRigidTransform3(GetX(Index), GetR(Index)));
	}

	template <typename T, int d, EGeometryParticlesSimType SimType>
	void TGeometryParticlesImp<T, d, SimType>::SetHandle(int32 Index, FGeometryParticleHandle* Handle)
	{
		Handle->SetSOALowLevel(this);
		MGeometryParticleHandle[Index] = AsAlwaysSerializable(Handle);
	}

	template <>
	void TGeometryParticlesImp<FRealSingle, 3, EGeometryParticlesSimType::Other>::SetHandle(int32 Index, FGeometryParticleHandle* Handle)
	{
		check(false);  // TODO: Implement EGeometryParticlesSimType::Other (cloth) particle serialization
	}

	template <>
	void TGeometryParticlesImp<FRealDouble, 3, EGeometryParticlesSimType::Other>::SetHandle(int32 Index, FGeometryParticleHandle* Handle)
	{
		check(false);  // TODO: Implement EGeometryParticlesSimType::Other (cloth) particle serialization
	}

	template<class T, int d, EGeometryParticlesSimType SimType>
	TGeometryParticlesImp<T, d, SimType>* TGeometryParticlesImp<T, d, SimType>::SerializationFactory(FChaosArchive& Ar, TGeometryParticlesImp<T,d,SimType>* Particles)
	{
		int8 ParticleType = Ar.IsLoading() ? 0 : (int8)Particles->ParticleType();
		Ar << ParticleType;
		switch ((EParticleType)ParticleType)
		{
		case EParticleType::Static: return Ar.IsLoading() ? new TGeometryParticlesImp<T, d, SimType>() : nullptr;
		case EParticleType::Kinematic: return Ar.IsLoading() ? new TKinematicGeometryParticlesImp<T, d, SimType>() : nullptr;
		case EParticleType::Rigid: return Ar.IsLoading() ? new TPBDRigidParticles<T, d>() : nullptr;
		case EParticleType::Clustered: return Ar.IsLoading() ? new TPBDRigidClusteredParticles<T, d>() : nullptr;
		default:
			check(false); return nullptr;
		}
	}
	
	template<>
	TGeometryParticlesImp<FRealSingle, 3, EGeometryParticlesSimType::Other>* TGeometryParticlesImp<FRealSingle, 3, EGeometryParticlesSimType::Other>::SerializationFactory(FChaosArchive& Ar, TGeometryParticlesImp<FRealSingle, 3, EGeometryParticlesSimType::Other>* Particles)
	{
		check(false);  // TODO: Implement EGeometryParticlesSimType::Other (cloth) particle serialization
		return nullptr;
	}

	template<>
	TGeometryParticlesImp<FRealDouble, 3, EGeometryParticlesSimType::Other>* TGeometryParticlesImp<FRealDouble, 3, EGeometryParticlesSimType::Other>::SerializationFactory(FChaosArchive& Ar, TGeometryParticlesImp<FRealDouble, 3, EGeometryParticlesSimType::Other>* Particles)
	{
		check(false);  // TODO: Implement EGeometryParticlesSimType::Other (cloth) particle serialization
		return nullptr;
	}

	template <typename T, int d, EGeometryParticlesSimType SimType>
	void TGeometryParticlesImp<T, d, SimType>::SerializeGeometryParticleHelper(FChaosArchive& Ar, TGeometryParticlesImp<T, d, EGeometryParticlesSimType::RigidBodySim>* GeometryParticles)
	{
		auto& SerializableGeometryParticles = AsAlwaysSerializableArray(GeometryParticles->MGeometryParticle);
		Ar << SerializableGeometryParticles;
	}

	template class TGeometryParticlesImp<FReal, 3, EGeometryParticlesSimType::RigidBodySim>;
	template class TGeometryParticlesImp<FRealSingle, 3, EGeometryParticlesSimType::Other>;
	template class TGeometryParticlesImp<FRealDouble, 3, EGeometryParticlesSimType::Other>;
}
