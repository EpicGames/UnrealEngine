// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/GeometryQueries.h"

#include "Chaos/Box.h"
#include "Chaos/Capsule.h"
#include "Chaos/CastingUtilities.h"
#include "Chaos/GJKShape.h"
#include "Chaos/ImplicitObject.h"
#include "Chaos/Sphere.h"
#include "Chaos/Sweeps.h"
#include "Chaos/Transform.h"

namespace Chaos
{
	void TransformSweepResultsToWorld(const bool bResult, const FReal Time, const bool bComputeMTD, const FImplicitObject& TestGeom, const FVec3& TestGeomLocation, const FVec3& LocalDir, const FVec3& LocalPosition, const FVec3& LocalNormal, const int32 FaceIndex,
		FVec3& OutWorldPosition, FVec3& OutWorldNormal, FVec3& OutWorldFaceNormal)
	{
		if (bResult && (Time >= 0 || bComputeMTD))
		{
			OutWorldPosition = TestGeomLocation + LocalPosition;
			OutWorldFaceNormal = TestGeom.FindGeometryOpposingNormal(LocalDir, FaceIndex, LocalNormal);
			OutWorldNormal = LocalNormal.GetSafeNormal(UE_KINDA_SMALL_NUMBER, FVec3::AxisVector(0));
		}
	}

	void TransformSweepResultsToWorld(const bool bResult, const FReal Time, const bool bComputeMTD, const FImplicitObject& TestGeom, const FRigidTransform3& TestGeomTM, const FVec3& LocalDir, const FVec3& LocalPosition, const FVec3& LocalNormal, const int32 FaceIndex,
		FVec3& OutWorldPosition, FVec3& OutWorldNormal, FVec3& OutWorldFaceNormal)
	{
		if (bResult && (Time >= 0 || bComputeMTD))
		{
			OutWorldPosition = TestGeomTM.TransformPositionNoScale(LocalPosition);
			OutWorldFaceNormal = TestGeomTM.TransformVectorNoScale(TestGeom.FindGeometryOpposingNormal(LocalDir, FaceIndex, LocalNormal));
			OutWorldNormal = TestGeomTM.TransformVectorNoScale(LocalNormal).GetSafeNormal(UE_KINDA_SMALL_NUMBER, FVec3::AxisVector(0));
		}
	}

	FGJKSphereSingleSIMD ComputeLocalSphere(const FSphere& Sphere, const FRigidTransform3& SphereTM, const FRigidTransform3& OtherTM)
	{
		const FVec3 SphereWorldLocation = FVec3(Sphere.GetCenterf()) + SphereTM.GetLocation();
		const FVec3 SphereLocation = OtherTM.InverseTransformPositionNoScale(SphereWorldLocation);
		return FGJKSphereSingleSIMD(SphereLocation, Sphere.GetRadiusf());
	}

	FGJKCapsuleSingleSIMD ComputeLocalCapsule(const FCapsule& Capsule, const FRigidTransform3& CapsuleTM, const FRigidTransform3& OtherTM)
	{
		const FRigidTransform3 CapsuleToBoxTM = CapsuleTM.GetRelativeTransformNoScale(OtherTM);
		const FVec3 CapsuleLocalAxis = CapsuleToBoxTM.TransformVectorNoScale(FVec3(Capsule.GetAxisf()));
		const FVec3 CapsuleLocalPointA = CapsuleToBoxTM.TransformPositionNoScale(FVec3(Capsule.GetX1f()));
		const FVec3 CapsuleLocalPointB = CapsuleLocalPointA + CapsuleLocalAxis * Capsule.GetHeightf();
		return FGJKCapsuleSingleSIMD(CapsuleLocalPointA, CapsuleLocalPointB, CapsuleLocalAxis, Capsule.GetRadiusf());
	}

	template <typename SweptShapeType, typename TestShapeType, typename TestObjectType>
	bool LocalSweep(const SweptShapeType& SweptShape, const TestShapeType& TestShape, const TestObjectType& TestObject, const FRigidTransform3& LocalToWorldTM,
		const FVec3& WorldSweepDir, const FReal Length, const FReal Thickness, const bool bComputeMTD,
		FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal)
	{
		const FVec3 LocalSweepDir = LocalToWorldTM.InverseTransformVectorNoScale(WorldSweepDir);
		// Note: The swept shape was fully transformed into the local space of the test object. 
		// This means the "start" is zero as we've offset the shape. This is typically a little faster than computing the "origin centered local object".
		const FVec3 Start = FVec3::ZeroVector;
		// Note: Technically any initial dir is valid. The unoptimized version uses A - B which is a little more expensive to calculate. The sweep dir should be a good initial guess as well.
		const FVec3 InitialDir = LocalSweepDir;

		const bool bResult = GJKRaycast2SameSpace<FReal>(TestShape, SweptShape, Start, LocalSweepDir, Length, OutTime, OutPosition, OutNormal, Thickness, bComputeMTD, InitialDir, Thickness);
		TransformSweepResultsToWorld(bResult, OutTime, bComputeMTD, TestObject, LocalToWorldTM, LocalSweepDir, OutPosition, OutNormal, OutFaceIndex, OutPosition, OutNormal, OutFaceNormal);
		return bResult;
	}

	bool SweepSphereVsSphere(const FSphere& SweptSphere, const FRigidTransform3& SweptSphereTM, const FSphere& TestSphere, const FRigidTransform3& TestSphereTM,
		const FVec3& SweepDir, const FReal Length, const FReal Thickness, const bool bComputeMTD,
		FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal)
	{
		const Sweeps::ESweepFlags Flags = bComputeMTD ? Sweeps::ESweepFlags::MTD : Sweeps::ESweepFlags::None;
		// Transform to the test shape's local space. Note: We can skip rotation which is much faster
		const FVec3 TestGeomLocalToWorld = TestSphereTM.GetLocation();
		const FVec3 SweptShapeLocalToWorld = SweptSphereTM.GetLocation();
		const FVec3 LocalSweepDir = SweepDir;
		const FVec3 LocalSweepStart = SweptSphere.GetCenterf() + SweptShapeLocalToWorld - TestGeomLocalToWorld;
		const FReal SweepRadius = Thickness + SweptSphere.GetRadiusf();

		const bool bResult = Sweeps::SweepSphereVsSphere(LocalSweepStart, LocalSweepDir, Length, SweepRadius, TestSphere.GetCenterf(), TestSphere.GetRadiusf(), Flags, OutTime, OutPosition, OutNormal);
		TransformSweepResultsToWorld(bResult, OutTime, bComputeMTD, TestSphere, TestGeomLocalToWorld, LocalSweepDir, OutPosition, OutNormal, OutFaceIndex, OutPosition, OutNormal, OutFaceNormal);
		return bResult;
	}

	bool SweepSphereVsBox(const FSphere& SweptSphere, const FRigidTransform3& SweptSphereTM, const TBox<FReal, 3>& TestBox, const FRigidTransform3& TestBoxTM, const FVec3& SweepDir, const FReal Length, const FReal Thickness, const bool bComputeMTD, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal)
	{
		const FGJKSphereSingleSIMD LocalSweptSphere = ComputeLocalSphere(SweptSphere, SweptSphereTM, TestBoxTM);
		return LocalSweep(LocalSweptSphere, TestBox, TestBox, TestBoxTM, SweepDir, Length, Thickness, bComputeMTD, OutTime, OutPosition, OutNormal, OutFaceIndex, OutFaceNormal);
	}

	bool SweepSphereVsCapsule(const FSphere& SweptSphere, const FRigidTransform3& SweptSphereTM, const FCapsule& TestCapsule, const FRigidTransform3& TestCapsuleTM,
		const FVec3& SweepDir, const FReal Length, const FReal Thickness, const bool bComputeMTD,
		FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal)
	{
		const Sweeps::ESweepFlags Flags = bComputeMTD ? Sweeps::ESweepFlags::MTD : Sweeps::ESweepFlags::None;
		// Bring both shapes into "world space" but offset so the capsule is at the origin
		const FVec3 SphereLocation = SweptSphereTM.GetLocation();
		const FVec3 CapsuleLocation = TestCapsuleTM.GetLocation();
		const FVec3 CapsuleWorldAxis = TestCapsuleTM.TransformVectorNoScale(FVec3(TestCapsule.GetAxisf()));
		const FVec3 CapsuleWorldX1 = TestCapsuleTM.TransformPositionNoScale(FVec3(TestCapsule.GetX1f())) - CapsuleLocation;
		const FVec3 CapsuleWorldX2 = TestCapsuleTM.TransformPositionNoScale(FVec3(TestCapsule.GetX2f())) - CapsuleLocation;
		const FVec3 WorldSweepDir = SweepDir;
		const FVec3 WorldSweepStart = SweptSphere.GetCenterf() + SphereLocation - CapsuleLocation;
		const FReal SweepRadius = Thickness + SweptSphere.GetRadiusf();

		const bool bResult = Sweeps::SweepSphereVsCapsule(WorldSweepStart, WorldSweepDir, Length, SweepRadius, TestCapsule.GetRadiusf(), TestCapsule.GetHeightf(), CapsuleWorldAxis, CapsuleWorldX1, CapsuleWorldX2, Flags, OutTime, OutPosition, OutNormal);
		TransformSweepResultsToWorld(bResult, OutTime, bComputeMTD, TestCapsule, CapsuleLocation, WorldSweepDir, OutPosition, OutNormal, OutFaceIndex, OutPosition, OutNormal, OutFaceNormal);
		return bResult;
	}

	bool SweepSphereVsConvex(const FSphere& SweptSphere, const FRigidTransform3& SweptSphereTM, const FImplicitObject& TestObject, const FRigidTransform3& TestObjectTM,
		const FVec3& SweepDir, const FReal Length, const FReal Thickness, const bool bComputeMTD,
		FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal)
	{
		const FGJKSphereSingleSIMD LocalSweptSphere = ComputeLocalSphere(SweptSphere, SweptSphereTM, TestObjectTM);
		return Utilities::CastHelperNoUnwrap(TestObject, TestObjectTM,
			[&LocalSweptSphere, &SweepDir, Length, Thickness, bComputeMTD, &OutTime, &OutPosition, &OutNormal, &OutFaceIndex, &OutFaceNormal]<typename T>(const T& TestObjectDowncast, const FRigidTransform3& TestObjectTM)
			{
				return LocalSweep(LocalSweptSphere, TestObjectDowncast, TestObjectDowncast, TestObjectTM, SweepDir, Length, Thickness, bComputeMTD, OutTime, OutPosition, OutNormal, OutFaceIndex, OutFaceNormal);
			});
	}

	bool SweepBoxVsSphere(const TBox<FReal, 3>& SweptBox, const FRigidTransform3& SweptBoxTM, const FSphere& TestSphere, const FRigidTransform3& TestSphereTM,
		const FVec3& SweepDir, const FReal Length, const FReal Thickness, const bool bComputeMTD,
		FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal)
	{
		const FGJKSphereSingleSIMD LocalTestSphere = ComputeLocalSphere(TestSphere, TestSphereTM, SweptBoxTM);
		return LocalSweep(SweptBox, LocalTestSphere, TestSphere, SweptBoxTM, SweepDir, Length, Thickness, bComputeMTD, OutTime, OutPosition, OutNormal, OutFaceIndex, OutFaceNormal);
	}

	bool SweepBoxVsCapsule(const TBox<FReal, 3>& SweptBox, const FRigidTransform3& SweptBoxTM, const FCapsule& TestCapsule, const FRigidTransform3& TestCapsuleTM, 
		const FVec3& SweepDir, const FReal Length, const FReal Thickness, const bool bComputeMTD, 
		FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal)
	{
		const FGJKCapsuleSingleSIMD LocalTestCapsule = ComputeLocalCapsule(TestCapsule, TestCapsuleTM, SweptBoxTM);
		return LocalSweep(SweptBox, LocalTestCapsule, TestCapsule, SweptBoxTM, SweepDir, Length, Thickness, bComputeMTD, OutTime, OutPosition, OutNormal, OutFaceIndex, OutFaceNormal);
	}

	bool SweepCapsuleVsSphere(const FCapsule& SweptCapsule, const FRigidTransform3& SweptCapsuleTM, const FSphere& TestSphere, const FRigidTransform3& TestSphereTM,
		const FVec3& SweepDir, const FReal Length, const FReal Thickness, const bool bComputeMTD,
		FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal)
	{
		// Convert this into sweeping the sphere against the capsule. This is identical except for the resultant impact info.
		const FVec3 WorldSweepDir = -SweepDir;
		bool bResult = SweepSphereVsCapsule(TestSphere, TestSphereTM, SweptCapsule, SweptCapsuleTM, WorldSweepDir, Length, Thickness, bComputeMTD, OutTime, OutPosition, OutNormal, OutFaceIndex, OutFaceNormal);

		// Now convert the results from being "on the capsule" to being "on the sphere"
		if (bResult)
		{
			const FVec3 SphereLocation = TestSphereTM.GetLocation();
			const FReal SphereRadius = TestSphere.GetRadiusf();

			OutNormal *= -1;
			// Reconstruct the position on the sphere in local space (we transform everything below)
			OutPosition = OutNormal * SphereRadius;
			TransformSweepResultsToWorld(bResult, OutTime, bComputeMTD, TestSphere, SphereLocation, WorldSweepDir, OutPosition, OutNormal, OutFaceIndex, OutPosition, OutNormal, OutFaceNormal);
		}
		return bResult;
	}

	bool SweepCapsuleVsBox(const FCapsule& SweptCapsule, const FRigidTransform3& SweptCapsuleTM, const TBox<FReal, 3>& TestBox, const FRigidTransform3& TestBoxTM, const FVec3& SweepDir, const FReal Length, const FReal Thickness, const bool bComputeMTD, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal)
	{
		const FGJKCapsuleSingleSIMD LocalSweptCapsule = ComputeLocalCapsule(SweptCapsule, SweptCapsuleTM, TestBoxTM);
		return LocalSweep(LocalSweptCapsule, TestBox, TestBox, TestBoxTM, SweepDir, Length, Thickness, bComputeMTD, OutTime, OutPosition, OutNormal, OutFaceIndex, OutFaceNormal);
	}

	bool SweepCapsuleVsCapsule(const FCapsule& SweptCapsule, const FRigidTransform3& SweptCapsuleTM, const FCapsule& TestCapsule, const FRigidTransform3& TestCapsuleTM, const FVec3& SweepDir, const FReal Length, const FReal Thickness, const bool bComputeMTD, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal)
	{
		const FGJKCapsuleSingleSIMD LocalSweptCapsule = ComputeLocalCapsule(SweptCapsule, SweptCapsuleTM, TestCapsuleTM);
		return LocalSweep(LocalSweptCapsule, TestCapsule, TestCapsule, TestCapsuleTM, SweepDir, Length, Thickness, bComputeMTD, OutTime, OutPosition, OutNormal, OutFaceIndex, OutFaceNormal);
	}

	bool SweepCapsuleVsConvex(const FCapsule& SweptCapsule, const FRigidTransform3& SweptCapsuleTM, const FImplicitObject& TestObject, const FRigidTransform3& TestObjectTM, const FVec3& SweepDir, const FReal Length, const FReal Thickness, const bool bComputeMTD, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal)
	{
		const FGJKCapsuleSingleSIMD LocalSweptCapsule = ComputeLocalCapsule(SweptCapsule, SweptCapsuleTM, TestObjectTM);
		return Utilities::CastHelperNoUnwrap(TestObject, TestObjectTM,
			[&LocalSweptCapsule, &SweepDir, Length, Thickness, bComputeMTD, &OutTime, &OutPosition, &OutNormal, &OutFaceIndex, &OutFaceNormal]<typename T>(const T& TestObjectDowncast, const FRigidTransform3& TestObjectTM)
			{
				return LocalSweep(LocalSweptCapsule, TestObjectDowncast, TestObjectDowncast, TestObjectTM, SweepDir, Length, Thickness, bComputeMTD, OutTime, OutPosition, OutNormal, OutFaceIndex, OutFaceNormal);
			});
	}
}
