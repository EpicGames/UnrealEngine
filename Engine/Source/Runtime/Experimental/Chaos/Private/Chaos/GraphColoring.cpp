// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/GraphColoring.h"
#include "Chaos/Array.h"
#include "ChaosLog.h"
#include "Chaos/Framework/Parallel.h"
#include "Containers/BitArray.h"
#include "Chaos/SoftsSolverParticlesRange.h"

template<typename DynamicParticlesType, int32 N, bool bAllDynamic>
static bool VerifyGraph(const TArray<TArray<int32>>& ColorGraph, const TArray<Chaos::TVector<int32, N>>& Graph, const DynamicParticlesType& InParticles)
{
	for (int32 Color = 0; Color < ColorGraph.Num(); ++Color)
	{
		TMap<int32, int32> ColorNodesToEdges;
		for (const int32 Edge : ColorGraph[Color])
		{
			for (int32 NIndex = 0; NIndex < N; ++NIndex)
			{
				const int32 Node = Graph[Edge][NIndex];
				const int32* const ExistingEdge = ColorNodesToEdges.Find(Node);
				if (ExistingEdge && *ExistingEdge != Edge)
				{
					UE_LOGF(LogChaos, Error, "Color %d has duplicate Node %d. First added for Edge %d, and now found for Edge %d", Color, Node, *ExistingEdge, Edge);
					return false;
				}
				if (bAllDynamic || InParticles.InvM(Node) != 0)
				{
					ColorNodesToEdges.Add(Node, Edge);
				}
			}
		}
	}
	return true;
}


//just verify if different element in each subcolor has intersecting nodes
template <typename T>
static bool VerifyGridBasedSubColoring(const TArray<TArray<int32>>& ElementsPerColor, const Chaos::TMPMGrid<T>& Grid, const TArray<TArray<int32>>& ConstraintsNodesSet, const TArray<TArray<TArray<int32>>>& ElementsPerSubColors)
{
	for (int32 i = 0; i < ElementsPerSubColors.Num(); i++)
	{
		for (int32 j = 0; j < ElementsPerSubColors[i].Num(); j++)
		{
			TSet<int32> CoveredGridNodes;
			//first gather all Grid nodes of an element:
			for (int32 k = 0; k < ElementsPerSubColors[i][j].Num(); k++)
			{
				int32 e = ElementsPerSubColors[i][j][k];
				for (int32 Node:ConstraintsNodesSet[e])
				{
					if (CoveredGridNodes.Contains(Node))
					{
						return false;
					}
					CoveredGridNodes.Emplace(Node);
				}
			}
		}
	}
	return true;

}

// Impl VerifyNodalColoring (TVec4 graph)
static bool VerifyNodalColoringImpl(const TArray<Chaos::TVec4<int32>>& Graph, int32 NumParticles, const TFunctionRef<bool(int32)>& IsDynamicParticle, const int32 GraphParticlesStart, const int32 GraphParticlesEnd, const TArray<TArray<int32>>& IncidentElements, const TArray<TArray<int32>>& IncidentElementsLocalIndex, const TArray<TArray<int32>>& ParticlesPerColor)
{
	TArray<bool> ParticleIsIncluded;
	ParticleIsIncluded.Init(false, NumParticles);
	for (int32 ColorIdx = 0; ColorIdx < ParticlesPerColor.Num(); ColorIdx++)
	{
		for (int32 ParticleInColorIdx = 0; ParticleInColorIdx < ParticlesPerColor[ColorIdx].Num(); ParticleInColorIdx++)
		{
			ParticleIsIncluded[ParticlesPerColor[ColorIdx][ParticleInColorIdx]] = true;
		}
	}

	for (int32 LocalParticleIdx = 0; LocalParticleIdx < GraphParticlesEnd - GraphParticlesStart; LocalParticleIdx++)
	{
		int32 ParticleIndex = LocalParticleIdx + GraphParticlesStart;
		if (IsDynamicParticle(ParticleIndex))
		{
			if (!ParticleIsIncluded[ParticleIndex])
			{
				return false;
			}
		}
	}

	TArray<int32> Particle2Incident;
	Particle2Incident.Init(INDEX_NONE, GraphParticlesEnd - GraphParticlesStart);
	for (int32 IncidentIdx = 0; IncidentIdx < IncidentElements.Num(); IncidentIdx++)
	{
		if (IncidentElements[IncidentIdx].Num() > 0)
		{
			Particle2Incident[Graph[IncidentElements[IncidentIdx][0]][IncidentElementsLocalIndex[IncidentIdx][0]]] = IncidentIdx;
		}
	}

	for (int32 ColorIdx = 0; ColorIdx < ParticlesPerColor.Num(); ColorIdx++)
	{
		TSet<int32> IncidentParticles;
		for (int32 ParticleInColorIdx = 0; ParticleInColorIdx < ParticlesPerColor[ColorIdx].Num(); ParticleInColorIdx++)
		{
			int32 ParticleIndex = ParticlesPerColor[ColorIdx][ParticleInColorIdx];
			TSet<int32> LocalIncidentParticles;
			for (int32 IncidentIdx = 0; IncidentIdx < IncidentElements[Particle2Incident[ParticleIndex]].Num(); IncidentIdx++)
			{
				int32 ElementIndex = IncidentElements[Particle2Incident[ParticleIndex]][IncidentIdx];
				for (int32 ElementLocalIdx = 0; ElementLocalIdx < 4; ElementLocalIdx++)
				{
					LocalIncidentParticles.Emplace(Graph[ElementIndex][ElementLocalIdx]);
				}
			}
			if (IncidentParticles.Contains(ParticleIndex))
			{
				return false;
			}
			else
			{
				for (int32 LocalParticle : LocalIncidentParticles)
				{
					IncidentParticles.Emplace(LocalParticle);
				}
			}
		}
	}

	return true;
}

// Impl VerifyNodalColoring (TArray graph)
static bool VerifyNodalColoringImpl(const TArray<TArray<int32>>& Graph, int32 NumParticles, const TFunctionRef<bool(int32)>& IsDynamicParticle, const int32 GraphParticlesStart, const int32 GraphParticlesEnd, const TArray<TArray<int32>>& IncidentElements, const TArray<TArray<int32>>& IncidentElementsLocalIndex, const TArray<TArray<int32>>& ParticlesPerColor)
{
	checkSlow(GraphParticlesStart <= GraphParticlesEnd);
	checkSlow(GraphParticlesEnd <= NumParticles);
	TArray<bool> ParticleIsIncluded;
	ParticleIsIncluded.Init(false, NumParticles);
	for (int32 ColorIdx = 0; ColorIdx < ParticlesPerColor.Num(); ColorIdx++)
	{
		for (int32 ParticleInColorIdx = 0; ParticleInColorIdx < ParticlesPerColor[ColorIdx].Num(); ParticleInColorIdx++)
		{
			ParticleIsIncluded[ParticlesPerColor[ColorIdx][ParticleInColorIdx]] = true;
		}
	}

	for (int32 LocalParticleIdx = 0; LocalParticleIdx < GraphParticlesEnd - GraphParticlesStart; LocalParticleIdx++)
	{
		int32 ParticleIndex = LocalParticleIdx + GraphParticlesStart;
		if (IsDynamicParticle(ParticleIndex) && IncidentElements[ParticleIndex].Num() > 0)
		{
			if (!ParticleIsIncluded[ParticleIndex])
			{
				return false;
			}
		}
	}

	TArray<int32> Particle2Incident;
	Particle2Incident.Init(INDEX_NONE, GraphParticlesEnd - GraphParticlesStart);
	for (int32 LocalParticleIdx = 0; LocalParticleIdx < GraphParticlesEnd - GraphParticlesStart; LocalParticleIdx++)
	{
		Particle2Incident[LocalParticleIdx] = LocalParticleIdx + GraphParticlesStart;
	}

	for (int32 ColorIdx = 0; ColorIdx < ParticlesPerColor.Num(); ColorIdx++)
	{
		TSet<int32> IncidentParticles;
		for (int32 ParticleInColorIdx = 0; ParticleInColorIdx < ParticlesPerColor[ColorIdx].Num(); ParticleInColorIdx++)
		{
			int32 ParticleIndex = ParticlesPerColor[ColorIdx][ParticleInColorIdx];
			TSet<int32> LocalIncidentParticles;
			for (int32 IncidentIdx = 0; IncidentIdx < IncidentElements[Particle2Incident[ParticleIndex]].Num(); IncidentIdx++)
			{
				int32 ElementIndex = IncidentElements[Particle2Incident[ParticleIndex]][IncidentIdx];
				for (int32 ElementLocalIdx = 0; ElementLocalIdx < Graph[ElementIndex].Num(); ElementLocalIdx++)
				{
					LocalIncidentParticles.Add(Graph[ElementIndex][ElementLocalIdx]);
				}
			}
			if (IncidentParticles.Contains(ParticleIndex))
			{
				return false;
			}
			else
			{
				for (int32 LocalParticle : LocalIncidentParticles)
				{
					IncidentParticles.Add(LocalParticle);
				}
			}
		}
	}

	return true;
}

// VerifyExtraNodalColoringImpl (3-graph)
static bool VerifyExtraNodalColoringImpl(int32 NumParticles, const TFunctionRef<bool(int32)>& IsDynamicParticle, const TArray<TArray<int32>>& StaticGraph, const TArray<TArray<int32>>& DynamicGraph, const TArray<TArray<int32>>& ExtraGraph, const TArray<TArray<int32>>& StaticIncidentElements, const TArray<TArray<int32>>& DynamicIncidentElements, const TArray<TArray<int32>>& ExtraIncidentElements, const TArray<int32>& ParticleColors, const TArray<TArray<int32>>& ParticlesPerColor)
{
	TBitArray ParticleIsIncluded(false, NumParticles);
	for (int32 ColorIdx = 0; ColorIdx < ParticlesPerColor.Num(); ColorIdx++)
	{
		for (int32 ParticleInColorIdx = 0; ParticleInColorIdx < ParticlesPerColor[ColorIdx].Num(); ParticleInColorIdx++)
		{
			ParticleIsIncluded[ParticlesPerColor[ColorIdx][ParticleInColorIdx]] = true;
		}
	}

	for (int32 ParticleIndex = 0; ParticleIndex < NumParticles; ParticleIndex++)
	{
		if (IsDynamicParticle(ParticleIndex) && (StaticIncidentElements[ParticleIndex].Num() > 0 || ExtraIncidentElements[ParticleIndex].Num() > 0 || DynamicIncidentElements[ParticleIndex].Num() > 0))
		{
			if (!ParticleIsIncluded[ParticleIndex])
			{
				return false;
			}
		}
	}

	for (int32 ColorIdx = 0; ColorIdx < ParticlesPerColor.Num(); ColorIdx++)
	{
		TSet<int32> IncidentParticles;
		IncidentParticles.Reserve(ParticlesPerColor[ColorIdx].Num());
		for (int32 ParticleInColorIdx = 0; ParticleInColorIdx < ParticlesPerColor[ColorIdx].Num(); ParticleInColorIdx++)
		{
			int32 ParticleIndex = ParticlesPerColor[ColorIdx][ParticleInColorIdx];
			TSet<int32> LocalIncidentParticles;
			for (int32 IncidentIdx = 0; IncidentIdx < StaticIncidentElements[ParticleIndex].Num(); IncidentIdx++)
			{
				int32 ElementIndex = StaticIncidentElements[ParticleIndex][IncidentIdx];
				for (int32 ElementLocalIdx = 0; ElementLocalIdx < StaticGraph[ElementIndex].Num(); ElementLocalIdx++)
				{
					LocalIncidentParticles.Add(StaticGraph[ElementIndex][ElementLocalIdx]);
				}
			}
			for (int32 IncidentIdx = 0; IncidentIdx < DynamicIncidentElements[ParticleIndex].Num(); IncidentIdx++)
			{
				int32 ElementIndex = DynamicIncidentElements[ParticleIndex][IncidentIdx];
				for (int32 ElementLocalIdx = 0; ElementLocalIdx < DynamicGraph[ElementIndex].Num(); ElementLocalIdx++)
				{
					LocalIncidentParticles.Add(DynamicGraph[ElementIndex][ElementLocalIdx]);
				}
			}
			if (ParticleIndex < ExtraIncidentElements.Num())
			{
				for (int32 IncidentIdx = 0; IncidentIdx < ExtraIncidentElements[ParticleIndex].Num(); IncidentIdx++)
				{
					int32 ElementIndex = ExtraIncidentElements[ParticleIndex][IncidentIdx];
					for (int32 ElementLocalIdx = 0; ElementLocalIdx < ExtraGraph[ElementIndex].Num(); ElementLocalIdx++)
					{
						LocalIncidentParticles.Add(ExtraGraph[ElementIndex][ElementLocalIdx]);
					}
				}
			}
			if (IncidentParticles.Contains(ParticleIndex))
			{
				return false;
			}
			else
			{
				for (int32 LocalParticle : LocalIncidentParticles)
				{
					IncidentParticles.Add(LocalParticle);
				}
			}
		}
	}

	return true;
}

// VerifyExtraNodalColoringImpl (2-graph) — delegates to 3-graph with empty DynamicGraph
static bool VerifyExtraNodalColoringImpl(int32 NumParticles, const TFunctionRef<bool(int32)>& IsDynamicParticle, const TArray<TArray<int32>>& Graph, const TArray<TArray<int32>>& ExtraGraph, const TArray<TArray<int32>>& IncidentElements, const TArray<TArray<int32>>& ExtraIncidentElements, const TArray<int32>& ParticleColors, const TArray<TArray<int32>>& ParticlesPerColor)
{
	// EmptyGraph doesn't need sizing — it's only indexed by constraint indices from DynamicIncidentElements, which are all empty
	const TArray<TArray<int32>> EmptyGraph;
	// Size must match NumParticles so the 3-graph version can safely index DynamicIncidentElements[ParticleIndex]
	TArray<TArray<int32>> EmptyIncidentElements;
	EmptyIncidentElements.SetNum(NumParticles);
	return VerifyExtraNodalColoringImpl(NumParticles, IsDynamicParticle, Graph, EmptyGraph, ExtraGraph, IncidentElements, EmptyIncidentElements, ExtraIncidentElements, ParticleColors, ParticlesPerColor);
}

template<typename DynamicParticlesType, int32 N, bool bAllDynamic>
TArray<TArray<int32>> Chaos::FGraphColoring::ComputeGraphColoringParticlesOrRange(const TArray<TVector<int32, N>>& Graph, const DynamicParticlesType& InParticles, const int32 GraphParticlesStart, const int32 GraphParticlesEnd)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FGraphColoring_ComputeGraphColoringN);
	checkSlow(GraphParticlesStart <= GraphParticlesEnd);
	checkSlow(GraphParticlesEnd <= (int32)InParticles.Size());

	TArray<TArray<int32>> ColorGraph;

	int32 MaxColor = INDEX_NONE;
	TArray<FColorSet> NodeUsedColorsSubArray;
	NodeUsedColorsSubArray.SetNum(GraphParticlesEnd - GraphParticlesStart);
	TArrayView<FColorSet> NodeUsedColors(NodeUsedColorsSubArray.GetData() - GraphParticlesStart, GraphParticlesEnd); // Only nodes starting with GraphParticlesStart are valid to access
	for (int32 EdgeIndex = 0; EdgeIndex < Graph.Num(); ++EdgeIndex)
	{
		// Find color that hasn't already been assigned to a node on this edge.
		int32 FirstFreeColor = 0;
		while (true)
		{
			bool bColorFound = false;
			for (int32 NIndex = 0; NIndex < N; ++NIndex)
			{
				const int32 NodeIndex = Graph[EdgeIndex][NIndex];
				if constexpr (!bAllDynamic)
				{
					const bool bIsParticleDynamic = InParticles.InvM(NodeIndex) != (decltype(InParticles.InvM(NodeIndex)))0.;
					if (!bIsParticleDynamic)
					{
						continue;
					}
				}
				if (NodeUsedColors[NodeIndex].Contains(FirstFreeColor))
				{
					bColorFound = true;
					break;
				}
			}
			if (!bColorFound)
			{
				break;
			}
			++FirstFreeColor;
		}
		MaxColor = FMath::Max(MaxColor, FirstFreeColor);
		if (ColorGraph.Num() <= MaxColor)
		{
			ColorGraph.SetNum(MaxColor + 1);
		}
		for (int32 NIndex = 0; NIndex < N; ++NIndex)
		{
			const int32 NodeIndex = Graph[EdgeIndex][NIndex];
			NodeUsedColors[NodeIndex].Add(FirstFreeColor);
		}
		ColorGraph[FirstFreeColor].Add(EdgeIndex);
	}
#if DO_GUARD_SLOW
	const bool bVerifyGraphResult = VerifyGraph<DynamicParticlesType, N, bAllDynamic>(ColorGraph, Graph, InParticles);
	checkSlow(bVerifyGraphResult);
#endif
	return ColorGraph;
}


template<typename T>
void Chaos::ComputeGridBasedGraphSubColoringPointer(const TArray<TArray<int32>>& ElementsPerColor, const TMPMGrid<T>& Grid, const int32 GridSize, TUniquePtr<TArray<TArray<int32>>>& PreviousColoring, const TArray<TArray<int32>>& ConstraintsNodesSet, TArray<TArray<TArray<int32>>>& ElementsPerSubColors) 
{
	typedef TArray<int32, TInlineAllocator<8>> FColorSet;

	bool bHaveInitialGuess = true;
	if (!PreviousColoring) 
	{
		bHaveInitialGuess = false;
		PreviousColoring = MakeUnique<TArray<TArray<int32>>>();
		PreviousColoring->SetNum(ElementsPerColor.Num());
	}
	ElementsPerSubColors.SetNum(ElementsPerColor.Num());

	PhysicsParallelFor(ElementsPerColor.Num(), [bHaveInitialGuess, &ElementsPerColor, &Grid, GridSize, &PreviousColoring, &ConstraintsNodesSet, &ElementsPerSubColors](const int32 Color)
		{
			const TArray<int32>& ColorElements = ElementsPerColor[Color];
			TArray<int32>& PreviousColoringElements = PreviousColoring->operator[](Color);
			if (!bHaveInitialGuess)
			{
				PreviousColoringElements.Init(INDEX_NONE, ColorElements.Num());
			}
			int32 NumNodes = GridSize;
			TArray<int32> ElementSubColors;
			ElementSubColors.Init(INDEX_NONE, ColorElements.Num());
			TArray<FColorSet> UsedColors;
			UsedColors.SetNum(NumNodes);
			int32 MaxColor = INDEX_NONE;
			for (int32 ElementIndex = 0; ElementIndex < ColorElements.Num(); ElementIndex++) 
			{
				int32 ColorToUse = 0;
				int32 Element = ColorElements[ElementIndex];
				// check initial guess:
				if (bHaveInitialGuess)
				{
					ColorToUse = PreviousColoringElements[ElementIndex];
					bool bColorFound = false;
					for (const int32 Node : ConstraintsNodesSet[Element]) 
					{
						if (UsedColors[Node].Contains(ColorToUse)) 
						{
							bColorFound = true;
							break;
						}
					}
					if (bColorFound) 
					{
						ColorToUse = 0;
					}
					else 
					{
						for (const int32 Node : ConstraintsNodesSet[Element])
						{
							UsedColors[Node].Emplace(ColorToUse);
						}
						ElementSubColors[ElementIndex] = ColorToUse;
					}
				}
				if (ElementSubColors[ElementIndex] == INDEX_NONE)
				{
					while (true) 
					{
						bool bColorFound = false;
						for (const int32 Node : ConstraintsNodesSet[Element]) 
						{
							if (UsedColors[Node].Contains(ColorToUse)) 
							{
								bColorFound = true;
								break;
							}
						}
						if (!bColorFound)
						{
							break;
						}
						ColorToUse++;
					}
					ElementSubColors[ElementIndex] = ColorToUse;
					for (const int32 Node : ConstraintsNodesSet[Element])
					{
						UsedColors[Node].Emplace(ColorToUse);
					}
				}

				// assign colors to previous guess for next timestep:
				MaxColor = FMath::Max(MaxColor, ColorToUse);
				PreviousColoringElements[ElementIndex] = ColorToUse;
			}

			ElementsPerSubColors[Color].Reset();
			ElementsPerSubColors[Color].SetNum(MaxColor + 1);

			for (int32 ElementIndex = 0; ElementIndex < ColorElements.Num(); ElementIndex++)
			{
				ElementsPerSubColors[Color][ElementSubColors[ElementIndex]].Emplace(ColorElements[ElementIndex]);
			}
		}, ElementsPerColor.Num() < 20);
	
	checkSlow(VerifyGridBasedSubColoring<T>(ElementsPerColor, Grid, ConstraintsNodesSet, ElementsPerSubColors));
}


template<typename T>
void Chaos::ComputeWeakConstraintsColoring(const TArray<TArray<int32>>& Indices, const TArray<TArray<int32>>& SecondIndices, const Chaos::TDynamicParticles<T, 3>& InParticles, TArray<TArray<int32>>& ConstraintsPerColor)
{
	Chaos::ComputeWeakConstraintsColoring(Indices, SecondIndices, (int32)InParticles.Size(), ConstraintsPerColor);
}

// Non-template verify helper (particles are unused in verification).
// bSeparateRanges=true when Indices and SecondIndices index into different particle ranges
// (cross-asset), so particle 0 in each range is a different particle.
static bool VerifyWeakConstraintsColoringByCount(const TArray<TArray<int32>>& Indices, const TArray<TArray<int32>>& SecondIndices, const TArray<TArray<int32>>& ConstraintsPerColor, bool bSeparateRanges)
{
	TArray<bool> ConstraintIsIncluded;
	ConstraintIsIncluded.Init(false, Indices.Num());
	for (int32 ColorIndex = 0; ColorIndex < ConstraintsPerColor.Num(); ColorIndex++)
	{
		for (int32 ConstraintIdx = 0; ConstraintIdx < ConstraintsPerColor[ColorIndex].Num(); ConstraintIdx++)
		{
			ConstraintIsIncluded[ConstraintsPerColor[ColorIndex][ConstraintIdx]] = true;
		}
	}
	for (int32 ConstraintIndex = 0; ConstraintIndex < Indices.Num(); ConstraintIndex++)
	{
		if (!ConstraintIsIncluded[ConstraintIndex])
		{
			return false;
		}
	}
	for (int32 ColorIndex = 0; ColorIndex < ConstraintsPerColor.Num(); ColorIndex++)
	{
		TSet<int32> CoveredParticles;
		TSet<int32> CoveredSecondParticles;
		TSet<int32>& SecondSet = bSeparateRanges ? CoveredSecondParticles : CoveredParticles;
		for (int32 ConstraintIdx = 0; ConstraintIdx < ConstraintsPerColor[ColorIndex].Num(); ConstraintIdx++)
		{
			for (int32 Node : Indices[ConstraintsPerColor[ColorIndex][ConstraintIdx]])
			{
				if (CoveredParticles.Contains(Node))
				{
					return false;
				}
				CoveredParticles.Emplace(Node);
			}
			if (SecondIndices.Num() > 0)
			{
				for (int32 Node : SecondIndices[ConstraintsPerColor[ColorIndex][ConstraintIdx]])
				{
					if (SecondSet.Contains(Node))
					{
						return false;
					}
					SecondSet.Emplace(Node);
				}
			}
		}
	}
	return true;
}

void Chaos::ComputeWeakConstraintsColoring(const TArray<TArray<int32>>& Indices, const TArray<TArray<int32>>& SecondIndices, int32 NumParticles, TArray<TArray<int32>>& ConstraintsPerColor)
{
	TArray<TSet<int32>> UsedColors;
	UsedColors.SetNum(NumParticles);

	ensure(Indices.Num() == SecondIndices.Num() || SecondIndices.Num() == 0);

	TArray<int32> ConstraintColors;
	ConstraintColors.Init(INDEX_NONE, Indices.Num());

	if (SecondIndices.Num() == 0)
	{
		for (int32 ConstraintIndex = 0; ConstraintIndex < Indices.Num(); ConstraintIndex++)
		{
			int32 ColorToUse = 0;
			while (true)
			{
				bool ColorFound = false;
				for (int32 Node : Indices[ConstraintIndex])
				{
					if (UsedColors[Node].Contains(ColorToUse))
					{
						ColorFound = true;
						break;
					}
				}
				if (!ColorFound)
				{
					break;
				}
				ColorToUse++;
			}
			ConstraintColors[ConstraintIndex] = ColorToUse;
			for (int32 Node : Indices[ConstraintIndex])
			{
				UsedColors[Node].Emplace(ColorToUse);
			}
		}
	}
	else
	{
		for (int32 ConstraintIndex = 0; ConstraintIndex < Indices.Num(); ConstraintIndex++)
		{
			int32 ColorToUse = 0;
			while (true)
			{
				bool ColorFound = false;
				for (int32 Node : Indices[ConstraintIndex])
				{
					if (UsedColors[Node].Contains(ColorToUse))
					{
						ColorFound = true;
						break;
					}
				}
				for (int32 Node : SecondIndices[ConstraintIndex])
				{
					if (UsedColors[Node].Contains(ColorToUse))
					{
						ColorFound = true;
						break;
					}
				}
				if (!ColorFound)
				{
					break;
				}
				ColorToUse++;
			}
			ConstraintColors[ConstraintIndex] = ColorToUse;
			for (int32 Node : Indices[ConstraintIndex])
			{
				UsedColors[Node].Emplace(ColorToUse);
			}
			for (int32 Node : SecondIndices[ConstraintIndex])
			{
				UsedColors[Node].Emplace(ColorToUse);
			}
		}
	}

	int32 NumColors = FMath::Max<int32>(ConstraintColors);
	ConstraintsPerColor.Empty();
	ConstraintsPerColor.SetNum(NumColors + 1);

	for (int32 ConstraintIndex = 0; ConstraintIndex < Indices.Num(); ConstraintIndex++)
	{
		ConstraintsPerColor[ConstraintColors[ConstraintIndex]].Emplace(ConstraintIndex);
	}

	checkSlow(VerifyWeakConstraintsColoringByCount(Indices, SecondIndices, ConstraintsPerColor, /*bSeparateRanges =*/ false));
}

void Chaos::ComputeWeakConstraintsColoring(const TArray<TArray<int32>>& Indices, const TArray<TArray<int32>>& SecondIndices, int32 NumParticles, int32 NumSecondParticles, TArray<TArray<int32>>& ConstraintsPerColor)
{
	TArray<TSet<int32>> ParticleUsedColors, SecondParticleUsedColors;
	ParticleUsedColors.SetNum(NumParticles);
	SecondParticleUsedColors.SetNum(NumSecondParticles);

	ensure(Indices.Num() == SecondIndices.Num() || SecondIndices.Num() == 0);

	TArray<int32> ConstraintColors;
	ConstraintColors.Init(INDEX_NONE, Indices.Num());

	if (SecondIndices.Num() == 0)
	{
		for (int32 ConstraintIndex = 0; ConstraintIndex < Indices.Num(); ConstraintIndex++)
		{
			int32 ColorToUse = 0;
			while (true)
			{
				bool ColorFound = false;
				for (int32 ParticleIndex : Indices[ConstraintIndex])
				{
					if (ParticleUsedColors[ParticleIndex].Contains(ColorToUse))
					{
						ColorFound = true;
						break;
					}
				}
				if (!ColorFound)
				{
					break;
				}
				ColorToUse++;
			}
			ConstraintColors[ConstraintIndex] = ColorToUse;
			for (int32 ParticleIndex : Indices[ConstraintIndex])
			{
				ParticleUsedColors[ParticleIndex].Emplace(ColorToUse);
			}
		}
	}
	else
	{
		for (int32 ConstraintIndex = 0; ConstraintIndex < Indices.Num(); ConstraintIndex++)
		{
			int32 ColorToUse = 0;
			while (true)
			{
				bool ColorFound = false;
				for (int32 ParticleIndex : Indices[ConstraintIndex])
				{
					if (ParticleUsedColors[ParticleIndex].Contains(ColorToUse))
					{
						ColorFound = true;
						break;
					}
				}
				for (int32 SecondParticleIndex : SecondIndices[ConstraintIndex])
				{
					if (SecondParticleUsedColors[SecondParticleIndex].Contains(ColorToUse))
					{
						ColorFound = true;
						break;
					}
				}
				if (!ColorFound)
				{
					break;
				}
				ColorToUse++;
			}
			ConstraintColors[ConstraintIndex] = ColorToUse;
			for (int32 ParticleIndex : Indices[ConstraintIndex])
			{
				ParticleUsedColors[ParticleIndex].Emplace(ColorToUse);
			}
			for (int32 SecondParticleIndex : SecondIndices[ConstraintIndex])
			{
				SecondParticleUsedColors[SecondParticleIndex].Emplace(ColorToUse);
			}
		}
	}

	int32 NumColors = FMath::Max<int32>(ConstraintColors);

	ConstraintsPerColor.Empty();
	ConstraintsPerColor.SetNum(NumColors + 1);

	for (int32 ConstraintIndex = 0; ConstraintIndex < Indices.Num(); ConstraintIndex++)
	{
		ConstraintsPerColor[ConstraintColors[ConstraintIndex]].Emplace(ConstraintIndex);
	}

	checkSlow(VerifyWeakConstraintsColoringByCount(Indices, SecondIndices, ConstraintsPerColor, /*bSeparateRanges =*/ true));
}

// ComputeNodalColoringImpl (TVec4 graph)
static TArray<TArray<int32>> ComputeNodalColoringImpl(const TArray<Chaos::TVec4<int32>>& Graph, int32 NumParticles, const TFunctionRef<bool(int32)>& IsDynamicParticle, const int32 GraphParticlesStart, const int32 GraphParticlesEnd, const TArray<TArray<int32>>& IncidentElements, const TArray<TArray<int32>>& IncidentElementsLocalIndex)
{
	checkSlow(GraphParticlesStart <= GraphParticlesEnd);
	checkSlow(GraphParticlesEnd <= NumParticles);
	TArray<TArray<int32>> ParticlesPerColor;

	TArray<int32> Particle2Incident;
	Particle2Incident.Init(INDEX_NONE, GraphParticlesEnd - GraphParticlesStart);
	//Assuming that offset of Graph is GraphParticlesStart
	for (int32 IncidentIdx = 0; IncidentIdx < IncidentElements.Num(); IncidentIdx++)
	{
		if (IncidentElements[IncidentIdx].Num() > 0)
		{
			Particle2Incident[Graph[IncidentElements[IncidentIdx][0]][IncidentElementsLocalIndex[IncidentIdx][0]] - GraphParticlesStart] = IncidentIdx;
		}
	}

	TArray<TSet<int32>> ElementColorsSet;
	ElementColorsSet.SetNum(Graph.Num());
	TArray<int32> ParticleColors;
	ParticleColors.Init(INDEX_NONE, GraphParticlesEnd - GraphParticlesStart);

	for (int32 LocalParticleIdx = 0; LocalParticleIdx < GraphParticlesEnd - GraphParticlesStart; LocalParticleIdx++)
	{
		int32 ParticleIndex = LocalParticleIdx + GraphParticlesStart;
		if (IsDynamicParticle(ParticleIndex) && Particle2Incident[ParticleIndex] != INDEX_NONE)
		{
			int32 ColorToUse = 0;
			while (true)
			{
				bool ColorFound = false;
				for (int32 IncidentIdx = 0; IncidentIdx < IncidentElements[Particle2Incident[ParticleIndex]].Num(); IncidentIdx++)
				{
					int32 ElementIndex = IncidentElements[Particle2Incident[ParticleIndex]][IncidentIdx];
					if (ElementColorsSet[ElementIndex].Contains(ColorToUse))
					{
						ColorFound = true;
						break;
					}
				}
				if (!ColorFound)
				{
					ParticleColors[LocalParticleIdx] = ColorToUse;
					for (int32 IncidentIdx = 0; IncidentIdx < IncidentElements[Particle2Incident[ParticleIndex]].Num(); IncidentIdx++)
					{
						int32 ElementIndex = IncidentElements[Particle2Incident[ParticleIndex]][IncidentIdx];
						ElementColorsSet[ElementIndex].Emplace(ColorToUse);
						if (ElementColorsSet[ElementIndex].Num() == 4)
						{
							ElementColorsSet[ElementIndex].Empty();
						}
					}
					break;
				}
				ColorToUse++;
			}
		}
	}

	int32 SizeColors = FMath::Max<int32>(ParticleColors);

	ParticlesPerColor.SetNum(0);
	ParticlesPerColor.Init(TArray<int32>(), SizeColors + 1);

	for (int32 LocalParticleIdx = 0; LocalParticleIdx < ParticleColors.Num(); LocalParticleIdx++)
	{
		if (ParticleColors[LocalParticleIdx] != INDEX_NONE)
		{
			ParticlesPerColor[ParticleColors[LocalParticleIdx]].Emplace(LocalParticleIdx + GraphParticlesStart);
		}
	}

	checkSlow(VerifyNodalColoringImpl(Graph, NumParticles, IsDynamicParticle, GraphParticlesStart, GraphParticlesEnd, IncidentElements, IncidentElementsLocalIndex, ParticlesPerColor));

	return ParticlesPerColor;
}

// TDynamicParticles delegator
template<typename T>
TArray<TArray<int32>> Chaos::ComputeNodalColoring(const TArray<TVec4<int32>>& Graph, const Chaos::TDynamicParticles<T, 3>& InParticles, const int32 GraphParticlesStart, const int32 GraphParticlesEnd, const TArray<TArray<int32>>& IncidentElements, const TArray<TArray<int32>>& IncidentElementsLocalIndex)
{
	return ComputeNodalColoringImpl(Graph, (int32)InParticles.Size(),
		[&InParticles](int32 Idx) { return InParticles.InvM(Idx) != (T)0.; },
		GraphParticlesStart, GraphParticlesEnd, IncidentElements, IncidentElementsLocalIndex);
}

// FSolverParticlesRange overload
TArray<TArray<int32>> Chaos::ComputeNodalColoring(const TArray<TVec4<int32>>& Graph, const Softs::FSolverParticlesRange& InParticles, const int32 GraphParticlesStart, const int32 GraphParticlesEnd, const TArray<TArray<int32>>& IncidentElements, const TArray<TArray<int32>>& IncidentElementsLocalIndex)
{
	return ComputeNodalColoringImpl(Graph, InParticles.Size(),
		[&InParticles](int32 Idx) { return InParticles.InvM(Idx) != (Softs::FSolverReal)0.; },
		GraphParticlesStart, GraphParticlesEnd, IncidentElements, IncidentElementsLocalIndex);
}


// ComputeNodalColoringImpl (TArray graph, ActiveView)
template<typename ParticleType>
static TArray<TArray<int32>> ComputeNodalColoringImpl(const TArray<TArray<int32>>& Graph, int32 NumParticles, const TFunctionRef<bool(int32)>& IsDynamicParticle, const int32 GraphParticlesStart, const int32 GraphParticlesEnd, const TArray<TArray<int32>>& IncidentElements, const TArray<TArray<int32>>& IncidentElementsLocalIndex, const Chaos::TPBDActiveView<ParticleType>* InParticleActiveView, TArray<int32>* ParticleColorsOut)
{
	checkSlow(GraphParticlesStart <= GraphParticlesEnd);
	checkSlow(GraphParticlesEnd <= NumParticles);
	TArray<TArray<int32>> ParticlesPerColor;

	TArray<int32> Particle2Incident;
	Particle2Incident.Init(INDEX_NONE, GraphParticlesEnd - GraphParticlesStart);
	for (int32 LocalParticleIdx = 0; LocalParticleIdx < GraphParticlesEnd - GraphParticlesStart; LocalParticleIdx++)
	{
		Particle2Incident[LocalParticleIdx] = LocalParticleIdx + GraphParticlesStart;
	}

	TArray<TSet<int32>> ElementColorsSet;
	ElementColorsSet.SetNum(Graph.Num());
	TArray<int32> ParticleColors;
	ParticleColors.Init(INDEX_NONE, GraphParticlesEnd - GraphParticlesStart);

	if (InParticleActiveView)
	{
		InParticleActiveView->SequentialFor([&IsDynamicParticle, &ElementColorsSet, &Graph, &GraphParticlesStart, &IncidentElements, &Particle2Incident, &ParticleColors]
		(const ParticleType& ActiveViewParticles, int32 ParticleIndex)
			{
				const int32 LocalParticleIdx = ParticleIndex - GraphParticlesStart;
				if (IsDynamicParticle(ParticleIndex) && Particle2Incident[ParticleIndex] != INDEX_NONE)
				{
					int32 ColorToUse = 0;
					while (true)
					{
						bool ColorFound = false;
						for (int32 IncidentIdx = 0; IncidentIdx < IncidentElements[Particle2Incident[ParticleIndex]].Num(); IncidentIdx++)
						{
							int32 ElementIndex = IncidentElements[Particle2Incident[ParticleIndex]][IncidentIdx];
							if (ElementColorsSet[ElementIndex].Contains(ColorToUse))
							{
								ColorFound = true;
								break;
							}
						}
						if (!ColorFound)
						{
							ParticleColors[LocalParticleIdx] = ColorToUse;
							for (int32 IncidentIdx = 0; IncidentIdx < IncidentElements[Particle2Incident[ParticleIndex]].Num(); IncidentIdx++)
							{
								int32 ElementIndex = IncidentElements[Particle2Incident[ParticleIndex]][IncidentIdx];
								ElementColorsSet[ElementIndex].Emplace(ColorToUse);
								if (ElementColorsSet[ElementIndex].Num() == Graph[ElementIndex].Num())
								{
									ElementColorsSet[ElementIndex].Empty();
								}
							}
							break;
						}
						ColorToUse++;
					}
				}
			});
	}
	else
	{
		for (int32 LocalParticleIdx = 0; LocalParticleIdx < GraphParticlesEnd - GraphParticlesStart; LocalParticleIdx++)
		{
			int32 ParticleIndex = LocalParticleIdx + GraphParticlesStart;
			if (IsDynamicParticle(ParticleIndex) && Particle2Incident[ParticleIndex] != INDEX_NONE)
			{
				int32 ColorToUse = 0;
				while (true)
				{
					bool ColorFound = false;
					for (int32 IncidentIdx = 0; IncidentIdx < IncidentElements[Particle2Incident[ParticleIndex]].Num(); IncidentIdx++)
					{
						int32 ElementIndex = IncidentElements[Particle2Incident[ParticleIndex]][IncidentIdx];
						if (ElementColorsSet[ElementIndex].Contains(ColorToUse))
						{
							ColorFound = true;
							break;
						}
					}
					if (!ColorFound)
					{
						ParticleColors[LocalParticleIdx] = ColorToUse;
						for (int32 IncidentIdx = 0; IncidentIdx < IncidentElements[Particle2Incident[ParticleIndex]].Num(); IncidentIdx++)
						{
							int32 ElementIndex = IncidentElements[Particle2Incident[ParticleIndex]][IncidentIdx];
							ElementColorsSet[ElementIndex].Emplace(ColorToUse);
							if (ElementColorsSet[ElementIndex].Num() == Graph[ElementIndex].Num())
							{
								ElementColorsSet[ElementIndex].Empty();
							}
						}
						break;
					}
					ColorToUse++;
				}
			}
		}
	}

	int32 SizeColors = FMath::Max<int32>(ParticleColors);

	ParticlesPerColor.SetNum(0);
	ParticlesPerColor.Init(TArray<int32>(), SizeColors + 1);

	for (int32 LocalParticleIdx = 0; LocalParticleIdx < ParticleColors.Num(); LocalParticleIdx++)
	{
		if (ParticleColors[LocalParticleIdx] != INDEX_NONE)
		{
			ParticlesPerColor[ParticleColors[LocalParticleIdx]].Emplace(LocalParticleIdx + GraphParticlesStart);
		}
	}

	if (ParticleColorsOut)
	{
		*ParticleColorsOut = ParticleColors;
	}

	checkSlow(VerifyNodalColoringImpl(Graph, NumParticles, IsDynamicParticle, GraphParticlesStart, GraphParticlesEnd, IncidentElements, IncidentElementsLocalIndex, ParticlesPerColor));

	return ParticlesPerColor;
}

// TDynamicParticles delegator
template<typename T, typename ParticleType>
TArray<TArray<int32>> Chaos::ComputeNodalColoring(const TArray<TArray<int32>>& Graph, const Chaos::TDynamicParticles<T, 3>& InParticles, const int32 GraphParticlesStart, const int32 GraphParticlesEnd, const TArray<TArray<int32>>& IncidentElements, const TArray<TArray<int32>>& IncidentElementsLocalIndex, const TPBDActiveView<ParticleType>* InParticleActiveView, TArray<int32>* ParticleColorsOut)
{
	return ComputeNodalColoringImpl(Graph, (int32)InParticles.Size(),
		[&InParticles](int32 Idx) { return InParticles.InvM(Idx) != (T)0.; },
		GraphParticlesStart, GraphParticlesEnd, IncidentElements, IncidentElementsLocalIndex, InParticleActiveView, ParticleColorsOut);
}

// FSolverParticlesRange overload
template<typename ParticleType>
TArray<TArray<int32>> Chaos::ComputeNodalColoring(const TArray<TArray<int32>>& Graph, const Softs::FSolverParticlesRange& InParticles, const int32 GraphParticlesStart, const int32 GraphParticlesEnd, const TArray<TArray<int32>>& IncidentElements, const TArray<TArray<int32>>& IncidentElementsLocalIndex, const TPBDActiveView<ParticleType>* InParticleActiveView, TArray<int32>* ParticleColorsOut)
{
	return ComputeNodalColoringImpl(Graph, InParticles.Size(),
		[&InParticles](int32 Idx) { return InParticles.InvM(Idx) != (Softs::FSolverReal)0.; },
		GraphParticlesStart, GraphParticlesEnd, IncidentElements, IncidentElementsLocalIndex, InParticleActiveView, ParticleColorsOut);
}


// ComputeExtraNodalColoringImpl (3-graph)
static void ComputeExtraNodalColoringImpl(const TArray<TArray<int32>>& StaticGraph, const TArray<TArray<int32>>& DynamicGraph, const TArray<TArray<int32>>& ExtraGraph, int32 NumParticles, const TFunctionRef<bool(int32)>& IsDynamicParticle, const TArray<TArray<int32>>& StaticIncidentElements, const TArray<TArray<int32>>& DynamicIncidentElements, const TArray<TArray<int32>>& ExtraIncidentElements, TArray<int32>& ParticleColors, TArray<TArray<int32>>& ParticlesPerColor)
{
	TBitArray ParticleIsAffected(false, NumParticles);
	for (int32 i = 0; i < ExtraIncidentElements.Num(); i++)
	{
		if (ExtraIncidentElements[i].Num() > 0)
		{
			ParticleIsAffected[i] = true;
		}
	}

	TSet<int32> UsedColors;
	for (int32 i = 0; i < ExtraIncidentElements.Num(); i++)
	{
		if (ParticleIsAffected[i])
		{
			int32 OriginalColor = ParticleColors[i];
			ParticleColors[i] = INDEX_NONE;
			if (OriginalColor != INDEX_NONE)
			{
				UsedColors.Reset();
				UsedColors.Reserve(ExtraIncidentElements[i].Num() + StaticIncidentElements[i].Num() + DynamicIncidentElements[i].Num());
				for (int32 j = 0; j < StaticIncidentElements[i].Num(); j++)
				{
					int32 ConstraintIndex = StaticIncidentElements[i][j];
					for (int32 ie = 0; ie < StaticGraph[ConstraintIndex].Num(); ie++)
					{
						UsedColors.Add(ParticleColors[StaticGraph[ConstraintIndex][ie]]);
					}
				}
				for (int32 j = 0; j < DynamicIncidentElements[i].Num(); j++)
				{
					int32 ConstraintIndex = DynamicIncidentElements[i][j];
					for (int32 ie = 0; ie < DynamicGraph[ConstraintIndex].Num(); ie++)
					{
						UsedColors.Add(ParticleColors[DynamicGraph[ConstraintIndex][ie]]);
					}
				}
				for (int32 j = 0; j < ExtraIncidentElements[i].Num(); j++)
				{
					int32 ConstraintIndex = ExtraIncidentElements[i][j];
					for (int32 ie = 0; ie < ExtraGraph[ConstraintIndex].Num(); ie++)
					{
						UsedColors.Add(ParticleColors[ExtraGraph[ConstraintIndex][ie]]);
					}
				}
				if (UsedColors.Contains(OriginalColor))
				{
					OriginalColor = 0;
					while (UsedColors.Contains(OriginalColor))
					{
						OriginalColor++;
					}
					ParticleColors[i] = OriginalColor;
				}
				else
				{
					ParticleColors[i] = OriginalColor;
					ParticleIsAffected[i] = false;
				}
			}
		}
	}


	Chaos::PhysicsParallelFor(ParticlesPerColor.Num(), [&](const int32 i)
		{
			int32 CurrentIndex = 0;
			for (int32 j = 0; j < ParticlesPerColor[i].Num(); j++)
			{
				if (!ParticleIsAffected[ParticlesPerColor[i][j]])
				{
					if (CurrentIndex != j)
					{
						ParticlesPerColor[i][CurrentIndex] = ParticlesPerColor[i][j];
					}
					CurrentIndex += 1;
				}
			}
			ParticlesPerColor[i].SetNum(CurrentIndex);
		}, ParticlesPerColor.Num() > 0 && ParticlesPerColor[0].Num() < 1000);

	ParticlesPerColor.SetNum(FMath::Max<int32>(ParticleColors) + 1);
	for (int32 i = 0; i < ExtraIncidentElements.Num(); i++)
	{
		if (ParticleIsAffected[i] && ParticleColors[i] != INDEX_NONE)
		{
			ParticlesPerColor[ParticleColors[i]].Emplace(i);
		}
	}

	checkSlow(VerifyExtraNodalColoringImpl(NumParticles, IsDynamicParticle, StaticGraph, DynamicGraph, ExtraGraph, StaticIncidentElements, DynamicIncidentElements, ExtraIncidentElements, ParticleColors, ParticlesPerColor));
}

// TDynamicParticles delegator
template<typename T>
void Chaos::ComputeExtraNodalColoring(const TArray<TArray<int32>>& StaticGraph, const TArray<TArray<int32>>& DynamicGraph, const TArray<TArray<int32>>& ExtraGraph, const Chaos::TDynamicParticles<T, 3>& InParticles, const TArray<TArray<int32>>& StaticIncidentElements, const TArray<TArray<int32>>& DynamicIncidentElements, const TArray<TArray<int32>>& ExtraIncidentElements, TArray<int32>& ParticleColors, TArray<TArray<int32>>& ParticlesPerColor)
{
	ComputeExtraNodalColoringImpl(StaticGraph, DynamicGraph, ExtraGraph, (int32)InParticles.Size(),
		[&InParticles](int32 Idx) { return InParticles.InvM(Idx) != (T)0.; },
		StaticIncidentElements, DynamicIncidentElements, ExtraIncidentElements, ParticleColors, ParticlesPerColor);
}

// FSolverParticlesRange overload
void Chaos::ComputeExtraNodalColoring(const TArray<TArray<int32>>& StaticGraph, const TArray<TArray<int32>>& DynamicGraph, const TArray<TArray<int32>>& ExtraGraph, const Softs::FSolverParticlesRange& InParticles, const TArray<TArray<int32>>& StaticIncidentElements, const TArray<TArray<int32>>& DynamicIncidentElements, const TArray<TArray<int32>>& ExtraIncidentElements, TArray<int32>& ParticleColors, TArray<TArray<int32>>& ParticlesPerColor)
{
	ComputeExtraNodalColoringImpl(StaticGraph, DynamicGraph, ExtraGraph, InParticles.Size(),
		[&InParticles](int32 Idx) { return InParticles.InvM(Idx) != (Softs::FSolverReal)0.; },
		StaticIncidentElements, DynamicIncidentElements, ExtraIncidentElements, ParticleColors, ParticlesPerColor);
}

// ComputeExtraNodalColoringImpl (2-graph) — delegates to 3-graph with empty DynamicGraph
static void ComputeExtraNodalColoringImpl(const TArray<TArray<int32>>& Graph, const TArray<TArray<int32>>& ExtraGraph, int32 NumParticles, const TFunctionRef<bool(int32)>& IsDynamicParticle, const TArray<TArray<int32>>& IncidentElements, const TArray<TArray<int32>>& ExtraIncidentElements, TArray<int32>& ParticleColors, TArray<TArray<int32>>& ParticlesPerColor)
{
	// EmptyGraph doesn't need sizing — it's only indexed by constraint indices from DynamicIncidentElements, which are all empty
	const TArray<TArray<int32>> EmptyGraph;
	// Size must match NumParticles so the 3-graph version can safely index DynamicIncidentElements[ParticleIndex]
	TArray<TArray<int32>> EmptyIncidentElements;
	EmptyIncidentElements.SetNum(NumParticles);
	ComputeExtraNodalColoringImpl(Graph, EmptyGraph, ExtraGraph, NumParticles, IsDynamicParticle, IncidentElements, EmptyIncidentElements, ExtraIncidentElements, ParticleColors, ParticlesPerColor);
}

// TDynamicParticles delegator (2-graph)
template<typename T>
void Chaos::ComputeExtraNodalColoring(const TArray<TArray<int32>>& Graph, const TArray<TArray<int32>>& ExtraGraph, const Chaos::TDynamicParticles<T, 3>& InParticles, const TArray<TArray<int32>>& IncidentElements, const TArray<TArray<int32>>& ExtraIncidentElements, TArray<int32>& ParticleColors, TArray<TArray<int32>>& ParticlesPerColor)
{
	ComputeExtraNodalColoringImpl(Graph, ExtraGraph, (int32)InParticles.Size(),
		[&InParticles](int32 Idx) { return InParticles.InvM(Idx) != (T)0.; },
		IncidentElements, ExtraIncidentElements, ParticleColors, ParticlesPerColor);
}

// FSolverParticlesRange overload (2-graph)
void Chaos::ComputeExtraNodalColoring(const TArray<TArray<int32>>& Graph, const TArray<TArray<int32>>& ExtraGraph, const Softs::FSolverParticlesRange& InParticles, const TArray<TArray<int32>>& IncidentElements, const TArray<TArray<int32>>& ExtraIncidentElements, TArray<int32>& ParticleColors, TArray<TArray<int32>>& ParticlesPerColor)
{
	ComputeExtraNodalColoringImpl(Graph, ExtraGraph, InParticles.Size(),
		[&InParticles](int32 Idx) { return InParticles.InvM(Idx) != (Softs::FSolverReal)0.; },
		IncidentElements, ExtraIncidentElements, ParticleColors, ParticlesPerColor);
}

#define UE_COMPUTE_GRAPH_COLORING_PARTICLES_OR_RANGE_N_HELPER(N, bAllDynamic) \
template CHAOS_API TArray<TArray<int32>> Chaos::FGraphColoring::ComputeGraphColoringParticlesOrRange<Chaos::TDynamicParticles<Chaos::FRealSingle, 3>, N, bAllDynamic>(const TArray<Chaos::TVector<int32, N>>&, const Chaos::TDynamicParticles<Chaos::FRealSingle, 3>&, const int32 GraphParticlesStart, const int32 GraphParticlesEnd); \
template CHAOS_API TArray<TArray<int32>> Chaos::FGraphColoring::ComputeGraphColoringParticlesOrRange<Chaos::TDynamicParticles<Chaos::FRealDouble, 3>, N, bAllDynamic>(const TArray<Chaos::TVector<int32, N>>&, const Chaos::TDynamicParticles<Chaos::FRealDouble, 3>&, const int32 GraphParticlesStart, const int32 GraphParticlesEnd); \
template CHAOS_API TArray<TArray<int32>> Chaos::FGraphColoring::ComputeGraphColoringParticlesOrRange<Chaos::Softs::FSolverParticles, N, bAllDynamic>(const TArray<Chaos::TVector<int32, N>>&, const Chaos::Softs::FSolverParticles&, const int32 GraphParticlesStart, const int32 GraphParticlesEnd); \
template CHAOS_API TArray<TArray<int32>> Chaos::FGraphColoring::ComputeGraphColoringParticlesOrRange<Chaos::Softs::FSolverParticlesRange, N, bAllDynamic>(const TArray<Chaos::TVector<int32, N>>&, const Chaos::Softs::FSolverParticlesRange&, const int32 GraphParticlesStart, const int32 GraphParticlesEnd);

UE_COMPUTE_GRAPH_COLORING_PARTICLES_OR_RANGE_N_HELPER(2, false)
UE_COMPUTE_GRAPH_COLORING_PARTICLES_OR_RANGE_N_HELPER(3, false)
UE_COMPUTE_GRAPH_COLORING_PARTICLES_OR_RANGE_N_HELPER(4, false)
UE_COMPUTE_GRAPH_COLORING_PARTICLES_OR_RANGE_N_HELPER(6, false)
UE_COMPUTE_GRAPH_COLORING_PARTICLES_OR_RANGE_N_HELPER(4, true)

template CHAOS_API void Chaos::ComputeGridBasedGraphSubColoringPointer(const TArray<TArray<int32>>& ElementsPerColor, const TMPMGrid<Chaos::FRealSingle>& Grid, const int32 GridSize, TUniquePtr<TArray<TArray<int32>>>& PreviousColoring, const TArray<TArray<int32>>& ConstraintsNodesSet, TArray<TArray<TArray<int32>>>& ElementsPerSubColors);
template CHAOS_API void Chaos::ComputeGridBasedGraphSubColoringPointer(const TArray<TArray<int32>>& ElementsPerColor, const TMPMGrid<Chaos::FRealDouble>& Grid, const int32 GridSize, TUniquePtr<TArray<TArray<int32>>>& PreviousColoring, const TArray<TArray<int32>>& ConstraintsNodesSet, TArray<TArray<TArray<int32>>>& ElementsPerSubColors);

template CHAOS_API void Chaos::ComputeWeakConstraintsColoring<Chaos::FRealSingle>(const TArray<TArray<int32>>& Indices, const TArray<TArray<int32>>& SecondIndices, const Chaos::TDynamicParticles<Chaos::FRealSingle, 3>& InParticles, TArray<TArray<int32>>& ConstraintsPerColor);
template CHAOS_API void Chaos::ComputeWeakConstraintsColoring<Chaos::FRealDouble>(const TArray<TArray<int32>>& Indices, const TArray<TArray<int32>>& SecondIndices, const Chaos::TDynamicParticles<Chaos::FRealDouble, 3>& InParticles, TArray<TArray<int32>>& ConstraintsPerColor);

template CHAOS_API TArray<TArray<int32>> Chaos::ComputeNodalColoring<Chaos::FRealSingle>(const TArray<TVec4<int32>>& Graph, const Chaos::TDynamicParticles<Chaos::FRealSingle, 3>& InParticles, const int32 GraphParticlesStart, const int32 GraphParticlesEnd, const TArray<TArray<int32>>& IncidentElements, const TArray<TArray<int32>>& IncidentElementsLocalIndex);
template CHAOS_API TArray<TArray<int32>> Chaos::ComputeNodalColoring<Chaos::FRealDouble>(const TArray<TVec4<int32>>& Graph, const Chaos::TDynamicParticles<Chaos::FRealDouble, 3>& InParticles, const int32 GraphParticlesStart, const int32 GraphParticlesEnd, const TArray<TArray<int32>>& IncidentElements, const TArray<TArray<int32>>& IncidentElementsLocalIndex);

template CHAOS_API TArray<TArray<int32>> Chaos::ComputeNodalColoring<Chaos::FRealSingle>(const TArray<TArray<int32>>& Graph, const Chaos::TDynamicParticles<Chaos::FRealSingle, 3>& InParticles, const int32 GraphParticlesStart, const int32 GraphParticlesEnd, const TArray<TArray<int32>>& IncidentElements, const TArray<TArray<int32>>& IncidentElementsLocalIndex, const TPBDActiveView<Chaos::Softs::FSolverParticles>* InParticleActiveView, TArray<int32>* ParticleColorsOut);
template CHAOS_API TArray<TArray<int32>> Chaos::ComputeNodalColoring<Chaos::FRealDouble>(const TArray<TArray<int32>>& Graph, const Chaos::TDynamicParticles<Chaos::FRealDouble, 3>& InParticles, const int32 GraphParticlesStart, const int32 GraphParticlesEnd, const TArray<TArray<int32>>& IncidentElements, const TArray<TArray<int32>>& IncidentElementsLocalIndex, const TPBDActiveView<Chaos::TDynamicParticles<Chaos::FRealDouble, 3>>* InParticleActiveView, TArray<int32>* ParticleColorsOut);

template CHAOS_API void Chaos::ComputeExtraNodalColoring<Chaos::FRealSingle>(const TArray<TArray<int32>>& Graph, const TArray<TArray<int32>>& ExtraGraph, const Chaos::TDynamicParticles<Chaos::FRealSingle, 3>& InParticles, const TArray<TArray<int32>>& IncidentElements, const TArray<TArray<int32>>& ExtraIncidentElements, TArray<int32>& ParticleColors, TArray<TArray<int32>>& ParticlesPerColor);
template CHAOS_API void Chaos::ComputeExtraNodalColoring<Chaos::FRealDouble>(const TArray<TArray<int32>>& Graph, const TArray<TArray<int32>>& ExtraGraph, const Chaos::TDynamicParticles<Chaos::FRealDouble, 3>& InParticles, const TArray<TArray<int32>>& IncidentElements, const TArray<TArray<int32>>& ExtraIncidentElements, TArray<int32>& ParticleColors, TArray<TArray<int32>>& ParticlesPerColor);

template CHAOS_API void Chaos::ComputeExtraNodalColoring<Chaos::FRealSingle>(const TArray<TArray<int32>>& StaticGraph, const TArray<TArray<int32>>& DynamicGraph, const TArray<TArray<int32>>& ExtraGraph, const Chaos::TDynamicParticles<Chaos::FRealSingle, 3>& InParticles, const TArray<TArray<int32>>& StaticIncidentElements, const TArray<TArray<int32>>& DynamicIncidentElements, const TArray<TArray<int32>>& ExtraIncidentElements, TArray<int32>& ParticleColors, TArray<TArray<int32>>& ParticlesPerColor);
template CHAOS_API void Chaos::ComputeExtraNodalColoring<Chaos::FRealDouble>(const TArray<TArray<int32>>& StaticGraph, const TArray<TArray<int32>>& DynamicGraph, const TArray<TArray<int32>>& ExtraGraph, const Chaos::TDynamicParticles<Chaos::FRealDouble, 3>& InParticles, const TArray<TArray<int32>>& StaticIncidentElements, const TArray<TArray<int32>>& DynamicIncidentElements, const TArray<TArray<int32>>& ExtraIncidentElements, TArray<int32>& ParticleColors, TArray<TArray<int32>>& ParticlesPerColor);

// FSolverParticlesRange explicit instantiation (only needed for the ActiveView templated overload)
template CHAOS_API TArray<TArray<int32>> Chaos::ComputeNodalColoring<Chaos::Softs::FSolverParticles>(const TArray<TArray<int32>>& Graph, const Chaos::Softs::FSolverParticlesRange& InParticles, const int32 GraphParticlesStart, const int32 GraphParticlesEnd, const TArray<TArray<int32>>& IncidentElements, const TArray<TArray<int32>>& IncidentElementsLocalIndex, const TPBDActiveView<Chaos::Softs::FSolverParticles>* InParticleActiveView, TArray<int32>* ParticleColorsOut);