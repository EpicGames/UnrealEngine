// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/HeightField.h"
#include "Chaos/Collision/ContactPoint.h"
#include "Chaos/Collision/ContactPointsMiscShapes.h"
#include "Chaos/Collision/MeshContactGenerator.h"
#include "Chaos/Collision/TriangleOverlap.h"
#include "Chaos/Collision/PBDCollisionConstraint.h"
#include "Chaos/CollisionOneShotManifolds.h"
#include "Chaos/Core.h"
#include "Chaos/Convex.h"
#include "Chaos/Box.h"
#include "Chaos/GJK.h"
#include "Chaos/ImplicitObjectScaled.h"
#include "Chaos/Capsule.h"
#include "Chaos/GeometryQueries.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformMisc.h"
#include "Chaos/Triangle.h"
#include "Chaos/TriangleRegister.h"

//UE_DISABLE_OPTIMIZATION

namespace Chaos
{
	namespace CVars
	{
		extern bool bCCDNewTargetDepthMode;
	}

	extern FRealSingle Chaos_Collision_EdgePrunePlaneDistance;

	extern bool bChaos_Collision_OneSidedHeightField;

	class FHeightfieldRaycastVisitor
	{
	public:
		FHeightfieldRaycastVisitor(const typename FHeightField::FDataType* InData, const FVec3& InStart, const FVec3& InDir, const FReal InThickness)
			: OutTime(TNumericLimits<FReal>::Max())
			, OutFaceIndex(INDEX_NONE)
			, GeomData(InData)
			, Start(InStart)
			, Dir(InDir)
			, Thickness(InThickness)
		{
			for (int Axis = 0; Axis < 3; ++Axis)
			{
				bParallel[Axis] = FMath::IsNearlyZero(Dir[Axis], (FReal)1.e-8);
				InvDir[Axis] = bParallel[Axis] ? 0 : 1 / Dir[Axis];
			}

			StartPointSimd = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(Start.X, Start.Y, Start.Z, 0.0));
			VectorRegister4Float DirSimd = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(Dir.X, Dir.Y, Dir.Z, 0.0));
			Parallel = VectorCompareLT(VectorAbs(DirSimd), GlobalVectorConstants::SmallNumber);
			InvDirSimd = VectorBitwiseNotAnd(Parallel, VectorDivide(VectorOne(), DirSimd));
		}

		enum class ERaycastType
		{
			Raycast,
			Sweep
		};

		FORCEINLINE_DEBUGGABLE bool VisitRaycast(int32 Payload, FReal CurrentLength)
		{
			const int32 SubY = Payload / (GeomData->NumCols - 1);
			const int32 FullIndex = Payload + SubY;

			// return if the triangle was hit or not 
			auto TestTriangle = [&](int32 FaceIndex, const FVec3& A, const FVec3& B, const FVec3& C) -> bool
			{
				FReal Time;
				FVec3 Normal;
				if (RayTriangleIntersection(Start, Dir, CurrentLength, A, B, C, Time, Normal))
				{
					if (Time < OutTime)
					{
						bool bHole = false;

						const int32 CellIndex = FaceIndex / 2;
						if (GeomData->MaterialIndices.IsValidIndex(CellIndex))
						{
							bHole = GeomData->MaterialIndices[CellIndex] == TNumericLimits<uint8>::Max();
						}

						if (!bHole)
						{
							OutPosition = Start + (Dir * Time);
							OutNormal = Normal;
							OutTime = Time;
							OutFaceIndex = FaceIndex;
							return true;
						}
					}
				}

				return false;
			};

			FVec3 Points[4];
			GeomData->GetPointsScaled(FullIndex, Points);
			bool bHit = false;
				// Test both triangles that are in this cell, as we could hit both in any order
				bHit |= TestTriangle(Payload * 2, Points[0], Points[1], Points[3]);
				bHit |= TestTriangle(Payload * 2 + 1, Points[0], Points[3], Points[2]);
			const bool bShouldContinueVisiting = !bHit;
			return bShouldContinueVisiting;
		}

		bool VisitSweep(int32 Payload, FReal& CurrentLength)
		{
			const int32 SubX = Payload % (GeomData->NumCols - 1);
			const int32 SubY = Payload / (GeomData->NumCols - 1);

			const int32 FullIndex = Payload + SubY;

			const FReal Radius = Thickness + UE_SMALL_NUMBER;
			const FReal Radius2 = Radius * Radius;

			// return if the triangle was hit or not 
			auto TestTriangle = [&](int32 FaceIndex, const FVec3& A, const FVec3& B, const FVec3& C) -> bool
			{
				const FVec3 AB = B - A;
				const FVec3 AC = C - A;

				FVec3 Normal = FVec3::CrossProduct(AB, AC);
				const FReal Len2 = Normal.SafeNormalize();

				if(!ensure(Len2 > UE_SMALL_NUMBER))
				{
					// Bad triangle, co-linear points or very thin
					return false;
				}

				const TPlane<FReal, 3> TrianglePlane(A, Normal);

				bool bIntersection = false;
				FVec3 ResultPosition(0);
				FVec3 ResultNormal(0);
				FReal Time = TNumericLimits<FReal>::Max();
				int32 DummyFaceIndex = INDEX_NONE;

				if(TrianglePlane.Raycast(Start, Dir, CurrentLength, Thickness, Time, ResultPosition, ResultNormal, DummyFaceIndex))
				{
					if(Time == 0)
					{
						// Initial overlap
						const FVec3 ClosestPtOnTri = FindClosestPointOnTriangle(TrianglePlane, A, B, C, Start);
						const FReal DistToTriangle2 = (Start - ClosestPtOnTri).SizeSquared();
						if(DistToTriangle2 <= Radius2)
						{
							OutTime = 0;
							OutPosition = ClosestPtOnTri;
							OutNormal = Normal;
							OutFaceIndex = FaceIndex;
							return true;
						}
					}
					else
					{
						const FVec3 ClosestPtOnTri = FindClosestPointOnTriangle(ResultPosition, A, B, C, ResultPosition);
						const FReal DistToTriangle2 = (ResultPosition - ClosestPtOnTri).SizeSquared();
						bIntersection = DistToTriangle2 <= UE_SMALL_NUMBER;
					}
				}

				if(!bIntersection)
				{
					//sphere is not immediately touching the triangle, but it could start intersecting the perimeter as it sweeps by
					FVec3 BorderPositions[3];
					FVec3 BorderNormals[3];
					FReal BorderTimes[3];
					bool bBorderIntersections[3];

					const FCapsule ABCapsule(A, B, Thickness);
					bBorderIntersections[0] = ABCapsule.Raycast(Start, Dir, CurrentLength, 0, BorderTimes[0], BorderPositions[0], BorderNormals[0], DummyFaceIndex);

					const FCapsule BCCapsule(B, C, Thickness);
					bBorderIntersections[1] = BCCapsule.Raycast(Start, Dir, CurrentLength, 0, BorderTimes[1], BorderPositions[1], BorderNormals[1], DummyFaceIndex);

					const FCapsule ACCapsule(A, C, Thickness);
					bBorderIntersections[2] = ACCapsule.Raycast(Start, Dir, CurrentLength, 0, BorderTimes[2], BorderPositions[2], BorderNormals[2], DummyFaceIndex);

					int32 MinBorderIdx = INDEX_NONE;
					FReal MinBorderTime = 0;

					for(int32 BorderIdx = 0; BorderIdx < 3; ++BorderIdx)
					{
						if(bBorderIntersections[BorderIdx])
						{
							if(!bIntersection || BorderTimes[BorderIdx] < MinBorderTime)
							{
								MinBorderTime = BorderTimes[BorderIdx];
								MinBorderIdx = BorderIdx;
								bIntersection = true;
							}
						}
					}

					if(MinBorderIdx != INDEX_NONE)
					{
						ResultNormal = BorderNormals[MinBorderIdx];
						ResultPosition = BorderPositions[MinBorderIdx] - ResultNormal * Thickness;

						if(Time == 0)
						{
							//we were initially overlapping with triangle plane so no normal was given. Compute it now
							FVec3 TmpNormal;
							const FReal SignedDistance = TrianglePlane.PhiWithNormal(Start, TmpNormal);
							ResultNormal = SignedDistance >= 0 ? TmpNormal : -TmpNormal;
						}

						Time = MinBorderTime;
					}
				}

				if(bIntersection)
				{
					if(Time < OutTime)
					{
						bool bHole = false;

						const int32 CellIndex = FaceIndex / 2;
						if(GeomData->MaterialIndices.IsValidIndex(CellIndex))
						{
							bHole = GeomData->MaterialIndices[CellIndex] == TNumericLimits<uint8>::Max();
						}

						if(!bHole)
						{
							OutPosition = ResultPosition;
							OutNormal = ResultNormal;
							OutTime = Time;
							OutFaceIndex = FaceIndex;
							CurrentLength = Time;
							return true;
						}
					}
				}

				return false;
			};

			FVec3 Points[4];
			FAABB3 CellBounds;
			GeomData->GetPointsAndBoundsScaled(FullIndex, Points, CellBounds);
			CellBounds.Thicken(Thickness);

			// Check cell bounds
			//todo: can do it without raycast
			FReal TOI;
			FVec3 HitPoint;
			bool bHit = false;
			if (CellBounds.RaycastFast(Start, Dir, InvDir, bParallel, CurrentLength, 1.0f / CurrentLength, TOI, HitPoint))
			{
				// Test both triangles that are in this cell, as we could hit both in any order
				bHit |= TestTriangle(Payload * 2, Points[0], Points[1], Points[3]);
				bHit |= TestTriangle(Payload * 2 + 1, Points[0], Points[3], Points[2]);
			}
			const bool bShouldContinueVisiting = !bHit;
			return bShouldContinueVisiting;
		}

		FReal OutTime;
		FVec3 OutPosition;
		FVec3 OutNormal;
		int32 OutFaceIndex;

		VectorRegister4Float StartPointSimd;
		VectorRegister4Float InvDirSimd;
		VectorRegister4Float Parallel;
		
		const typename FHeightField::FDataType* GeomData;
	private:

		FVec3 Start;
		FVec3 Dir;
		FVec3 InvDir;
		bool  bParallel[3];
		FReal Thickness;
	};

	template<typename GeomQueryType>
	class THeightfieldSweepVisitor
	{
	public:

		THeightfieldSweepVisitor(const typename FHeightField::FDataType* InData, const GeomQueryType& InQueryGeom, const FRigidTransform3& InStartTM, const FVec3& InDir, const FReal InThickness, bool InComputeMTD)
			: OutTime(TNumericLimits<FReal>::Max())
			, OutFaceIndex(INDEX_NONE)
			, HfData(InData)
			, StartTM(InStartTM)
			, OtherGeom(InQueryGeom)
			, Dir(InDir)
			, Thickness(InThickness)
			, bComputeMTD(InComputeMTD)
		{
			const FAABB3 QueryBounds = InQueryGeom.BoundingBox();
			StartPoint = StartTM.TransformPositionNoScale(QueryBounds.Center());
			//Need inflation to consider rotation, there's probably a cheaper way to do this
			FTransform RotationTM(StartTM.GetRotation(), FVector(0));
			const FAABB3 RotatedBounds = QueryBounds.TransformedAABB(RotationTM);
			Inflation3D = RotatedBounds.Extents() * 0.5 + FVec3(Thickness);
			for (int Axis = 0; Axis < 3; ++Axis)
			{
				bParallel[Axis] = FMath::IsNearlyZero(Dir[Axis], (FReal)1.e-8);
				InvDir[Axis] = bParallel[Axis] ? 0 : 1 / Dir[Axis];
			}

			StartPointSimd = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(StartPoint.X, StartPoint.Y, StartPoint.Z, 0.0));
			DirSimd = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(Dir.X, Dir.Y, Dir.Z, 0.0));
			Parallel = VectorCompareLT(VectorAbs(DirSimd), GlobalVectorConstants::SmallNumber);
			InvDirSimd = VectorBitwiseNotAnd(Parallel, VectorDivide(VectorOne(), DirSimd));
			Inflation3DSimd = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(Inflation3D.X, Inflation3D.Y, Inflation3D.Z, 0.0));
			Inflation3DSimd = VectorAbs(Inflation3DSimd);

		}

		bool VisitSweep(int32 Payload, FReal& CurrentLength)
		{
			const int32 SubX = Payload % (HfData->NumCols - 1);
			const int32 SubY = Payload / (HfData->NumCols - 1);

			const int32 FullIndex = Payload + SubY;

			auto TestTriangle = [&](int32 FaceIndex, const VectorRegister4Float& A, const VectorRegister4Float& B, const VectorRegister4Float& C) -> bool
			{
				//Convert into local space of A to get better precision
				FTriangleRegister Triangle(VectorZero(), VectorSubtract(B, A), VectorSubtract(C, A));

				const UE::Math::TQuat<FReal>& RotationDouble = StartTM.GetRotation();
				VectorRegister4Float Rotation = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(RotationDouble.X, RotationDouble.Y, RotationDouble.Z, RotationDouble.W));
				// Normalize rotation
				Rotation = VectorNormalizeSafe(Rotation, GlobalVectorConstants::Float0001);

				const UE::Math::TVector<FReal>& TranslationDouble = StartTM.GetTranslation();
				VectorRegister4Float Translation = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(TranslationDouble.X, TranslationDouble.Y, TranslationDouble.Z, 0.0));
				Translation = VectorSubtract(Translation, A);

				FRealSingle Time;
				VectorRegister4Float OutPositionSimd, OutNormalSimd;
				if(GJKRaycast2ImplSimd(Triangle, OtherGeom, Rotation, Translation, DirSimd, static_cast<FRealSingle>(CurrentLength), Time, OutPositionSimd, OutNormalSimd, bComputeMTD, GlobalVectorConstants::Float1000))
				{
					if(Time < OutTime)
					{
						bool bHole = false;

						const int32 CellIndex = FaceIndex / 2;
						if(HfData->MaterialIndices.IsValidIndex(CellIndex))
						{
							bHole = HfData->MaterialIndices[CellIndex] == TNumericLimits<uint8>::Max();
						}

						if(!bHole)
						{
							alignas(16) FRealSingle OutFloat[4];
							VectorStoreAligned(OutNormalSimd, OutFloat);
							OutNormal.X = OutFloat[0];
							OutNormal.Y = OutFloat[1];
							OutNormal.Z = OutFloat[2];

							OutPositionSimd = VectorAdd(OutPositionSimd, A);
							VectorStoreAligned(OutPositionSimd, OutFloat);
							OutPosition.X = OutFloat[0];
							OutPosition.Y = OutFloat[1];
							OutPosition.Z = OutFloat[2];

							OutTime = Time;
							OutFaceIndex = FaceIndex;

							// If there's an initial overlap, we can't terminate right away as the first hit might not be the "best" (min distance). 
							// We have to process all initial overlaps no matter what, however we can still update CurrentLength to not process non-initial overlaps.
							if(Time <= 0) //initial overlap or MTD, so stop
							{
								// This is incorrect. To prevent objects pushing through the surface of the heightfield
								// we adopt the triangle normal but this leaves us with an incorrect MTD from the GJK call
								// above. #TODO possibly re-do GJK with a plane, or some geom vs.plane special case to solve
								// both triangles as planes 
								const VectorRegister4Float AB = VectorSubtract(B, A);
								const VectorRegister4Float AC = VectorSubtract(C, A);

								VectorRegister4Float TriNormal = VectorCross(AB, AC);
								TriNormal = VectorNormalize(TriNormal);

								VectorStoreAligned(TriNormal, OutFloat);
								OutNormal.X = OutFloat[0];
								OutNormal.Y = OutFloat[1];
								OutNormal.Z = OutFloat[2];

								// In an ideal world, we'd update CurrentLength to 0 so as to prevent getting any non-zero TOI intersections,
								// however GJKRaycast does not allow a ray length of 0. We could switch to GJKPenetration, 
								// however there is currently no SIMD implementation. To work around this, we can set CurrentLength to a small number instead.
								// Doing this will always produce the correct result (in the edge case we get a TOI between 0 and espilon, we'll still pick the smallest TOI).
								// This might be slightly slower in this edge case as GJK might've been able to terminate earlier.
								// #TODO Replace this pathway with a call to GJKPenetration when a SIMD version is created.
								CurrentLength = UE_KINDA_SMALL_NUMBER;
								return true;
							}

							CurrentLength = Time;
						}
					}
				}

				return true;
			};

			VectorRegister4Float Points[4];
			FAABBVectorized CellBounds;
			HfData->GetPointsAndBoundsScaledSimd(FullIndex, Points, CellBounds);
			CellBounds.Thicken(Inflation3DSimd);

			if (CellBounds.RaycastFast(StartPointSimd, InvDirSimd, Parallel, VectorSetFloat1(static_cast<FRealSingle>(CurrentLength))))
			{
				bool bContinue = TestTriangle(Payload * 2, Points[0], Points[1], Points[3]);
				if (bContinue)
				{
					TestTriangle(Payload * 2 + 1, Points[0], Points[3], Points[2]);
				}
			}
			// There is no way to early terminate, even if there is an initial overlap since another cell might have a "better" initial overlap.
			return true;
		}

		FReal OutTime;
		FVec3 OutPosition;
		FVec3 OutNormal;
		int32 OutFaceIndex;

		VectorRegister4Float StartPointSimd;
		VectorRegister4Float InvDirSimd;
		VectorRegister4Float DirSimd;
		VectorRegister4Float Parallel;
		VectorRegister4Float Inflation3DSimd;

	private:

		const typename FHeightField::FDataType* HfData;
		const FRigidTransform3 StartTM;
		const GeomQueryType& OtherGeom;
		const FVec3& Dir;
		FVec3 InvDir;
		bool  bParallel[3];
		const FReal Thickness;
		bool bComputeMTD;
		FVec3 StartPoint;
		FVec3 Inflation3D;

	};

	template<typename GeomQueryType>
	class THeightfieldSweepVisitorCCD
	{
	public:

		THeightfieldSweepVisitorCCD(const typename FHeightField::FDataType* InData, const GeomQueryType& InQueryGeom, const FAABB3& RotatedQueryBounds, const FRigidTransform3& InStartTM, const FVec3& InDir, const FReal Length, const FReal InIgnorePenetration, const FReal InTargetPenetration)
			: OutDistance(TNumericLimits<FReal>::Max())
			, OutPhi(TNumericLimits<FReal>::Max())
			, OutPosition(0)
			, OutNormal(0)
			, OutFaceIndex(INDEX_NONE)
			, HfData(InData)
			, StartTM(InStartTM)
			, OtherGeom(InQueryGeom)
			, IgnorePenetration(InIgnorePenetration)
			, TargetPenetration(InTargetPenetration)
		{
			const FVec3 StartPoint = StartTM.GetTranslation();
			const FVec3 BoundsStartPoint = StartPoint + RotatedQueryBounds.Center();
			const FVec3 Inflation3D = RotatedQueryBounds.Extents() * FReal(0.5);

			const VectorRegister4Float LengthSimd = MakeVectorRegisterFloatFromDouble(VectorSetFloat1(Length));
			BoundsStartPointSimd = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(BoundsStartPoint.X, BoundsStartPoint.Y, BoundsStartPoint.Z, 0.0));
			StartPointSimd = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(StartPoint.X, StartPoint.Y, StartPoint.Z, 0.0));
			DirSimd = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(InDir.X, InDir.Y, InDir.Z, 0.0));
			EndPointSimd = VectorAdd(StartPointSimd, VectorMultiply(DirSimd, LengthSimd));
			RotationSimd = MakeVectorRegisterFloatFromDouble(StartTM.GetRotationRegister());
			ParallelSimd = VectorCompareLT(VectorAbs(DirSimd), GlobalVectorConstants::SmallNumber);
			InvDirSimd = VectorBitwiseNotAnd(ParallelSimd, VectorDivide(VectorOne(), DirSimd));
			Inflation3DSimd = MakeVectorRegisterFloatFromDouble(VectorAbs(MakeVectorRegister(Inflation3D.X, Inflation3D.Y, Inflation3D.Z, 0.0)));

			// @todo(chaos): do we really need this?
			RotationSimd = VectorNormalizeSafe(RotationSimd, GlobalVectorConstants::Float0001);
		}

		bool VisitSweep(int32 CellIndex, FReal& CurrentLength)
		{
			// Skip holes
			if (HfData->MaterialIndices.IsValidIndex(CellIndex))
			{
				const bool bHole = (HfData->MaterialIndices[CellIndex] == TNumericLimits<uint8>::Max());
				if (bHole)
				{
					return true;
				}
			}

			auto TestTriangle = [&](int32 FaceIndex, const VectorRegister4Float& A, const VectorRegister4Float& B, const VectorRegister4Float& C) -> bool
			{
				// Convert into A-relative positions to get better precision
				FTriangleRegister Triangle(VectorZero(), VectorSubtract(B, A), VectorSubtract(C, A));
				VectorRegister4Float TranslationSimd = MakeVectorRegisterFloatFromDouble(VectorSubtract(StartTM.GetTranslationRegister(), A));

				// If we are definitely penetrating by less than IgnorePenetration at T=1, we can ignore this triangle
				// Checks the query geometry support point along the normal at the sweep end point
				// NOTE: We are using SupportCore, so we need to add Radius to the early-out distance check. We could just use Support
				// but this way avoids an extra sqrt. We don't modify IgnorePenetration in the ctor because it gets used below.
				VectorRegister4Float RadiusV = VectorSetFloat1(OtherGeom.GetRadiusf());
				VectorRegister4Float IgnorePenetrationSimd = VectorSubtract(MakeVectorRegisterFloatFromDouble(VectorSetFloat1(IgnorePenetration)), RadiusV);
				VectorRegister4Float TriangleNormalSimd = VectorNormalize(VectorCross(VectorSubtract(B, A), VectorSubtract(C, A)));
				VectorRegister4Float OtherTriangleNormalSimd = VectorQuaternionInverseRotateVector(RotationSimd, TriangleNormalSimd);
				VectorRegister4Float OtherTrianglePositionSimd = VectorQuaternionInverseRotateVector(RotationSimd, VectorSubtract(A, EndPointSimd));
				VectorRegister4Float OtherSupportSimd = OtherGeom.SupportCoreSimd(VectorNegate(OtherTriangleNormalSimd), 0);
				VectorRegister4Float MaxDepthSimd = VectorDot3(VectorSubtract(OtherTrianglePositionSimd, OtherSupportSimd), OtherTriangleNormalSimd);
				if (VectorMaskBits(VectorCompareLT(MaxDepthSimd, IgnorePenetrationSimd)))
				{
					return true;
				}

				FRealSingle Distance;
				VectorRegister4Float PositionSimd, NormalSimd;
				if (GJKRaycast2ImplSimd(Triangle, OtherGeom, RotationSimd, TranslationSimd, DirSimd, FRealSingle(CurrentLength), Distance, PositionSimd, NormalSimd, true, GlobalVectorConstants::Float1000))
				{
					// Convert back from A-relative positions
					PositionSimd = VectorAdd(PositionSimd, A);

					const VectorRegister4Float DirDotNormalSimd = VectorDot3(DirSimd, NormalSimd);
					FReal DirDotNormal;
					VectorStoreFloat1(DirDotNormalSimd, &DirDotNormal);

					// Calculate the time to reach a depth of TargetPenetration
					FReal TargetTOI, TargetPhi;
					ComputeSweptContactTOIAndPhiAtTargetPenetration(DirDotNormal, CurrentLength, FReal(Distance), IgnorePenetration, TargetPenetration, TargetTOI, TargetPhi);

					// If this hit is closest, store the result
					const FReal TargetDistance = TargetTOI * CurrentLength;
					if (TargetDistance < CurrentLength)
					{
						CurrentLength = TargetDistance;

						VectorStoreFloat3(MakeVectorRegisterDouble(NormalSimd), &OutNormal);
						VectorStoreFloat3(MakeVectorRegisterDouble(PositionSimd), &OutPosition);
						OutDistance = TargetDistance;
						OutPhi = TargetPhi;
						OutFaceIndex = FaceIndex;
					}
				}

				return (CurrentLength > 0);
			};


			VectorRegister4Float Points[4];
			FAABBVectorized CellBounds;
			const int32 VertexIndex = HfData->CellIndexToVertexIndex(CellIndex);
			HfData->GetPointsAndBoundsScaledSimd(VertexIndex, Points, CellBounds);
			CellBounds.Thicken(Inflation3DSimd);

			if (CellBounds.RaycastFast(BoundsStartPointSimd, InvDirSimd, ParallelSimd, VectorSetFloat1(static_cast<FRealSingle>(CurrentLength))))
			{
				const int32 TriIndex0 = 2 * CellIndex + 0;
				const int32 TriIndex1 = 2 * CellIndex + 1;
				if (!TestTriangle(TriIndex0, Points[0], Points[1], Points[3]))
				{
					return false;
				}
				if (!TestTriangle(TriIndex1, Points[0], Points[3], Points[2]))
				{
					return false;
				}
			}

			return true;
		}

		FReal OutDistance;
		FReal OutPhi;
		FVec3 OutPosition;
		FVec3 OutNormal;
		int32 OutFaceIndex;

	private:
		const typename FHeightField::FDataType* HfData;
		const FRigidTransform3 StartTM;
		const GeomQueryType& OtherGeom;
		FReal IgnorePenetration;
		FReal TargetPenetration;

		VectorRegister4Float BoundsStartPointSimd;
		VectorRegister4Float StartPointSimd;
		VectorRegister4Float EndPointSimd;
		VectorRegister4Float InvDirSimd;
		VectorRegister4Float DirSimd;
		VectorRegister4Float RotationSimd;
		VectorRegister4Float ParallelSimd;
		VectorRegister4Float Inflation3DSimd;
	};


	template<typename BufferType, typename ToRealFuncType>
	void BuildGeomData(TArrayView<BufferType> BufferView, TArrayView<const uint8> MaterialIndexView, int32 NumRows, int32 NumCols, const FVec3& InScale, ToRealFuncType ToRealFunc, typename FHeightField::FDataType& OutData, FAABB3& OutBounds)
	{
		using FDataType = typename FHeightField::FDataType;

		const bool bHaveMaterials = MaterialIndexView.Num() > 0;
		const bool bOnlyDefaultMaterial = MaterialIndexView.Num() == 1;
		ensure(BufferView.Num() == NumRows * NumCols);
		ensure(NumRows > 1);
		ensure(NumCols > 1);

		// Populate data.
		const int32 NumHeights = BufferView.Num();
		OutData.Heights.SetNum(NumHeights);

		OutData.NumRows = static_cast<uint16>(NumRows);
		OutData.NumCols = static_cast<uint16>(NumCols);
		OutData.MinValue = ToRealFunc(BufferView[0]);
		OutData.MaxValue = ToRealFunc(BufferView[0]);
		OutData.Scale = InScale;
		OutData.ScaleSimd = MakeVectorRegisterFloatFromDouble(MakeVectorRegisterDouble(InScale.X, InScale.Y, InScale.Z, 0.0));

		for(int32 HeightIndex = 1; HeightIndex < NumHeights; ++HeightIndex)
		{
			const FReal CurrHeight = ToRealFunc(BufferView[HeightIndex]);

			if(CurrHeight > OutData.MaxValue)
			{
				OutData.MaxValue = CurrHeight;
			}
			else if(CurrHeight < OutData.MinValue)
			{
				OutData.MinValue = CurrHeight;
			}
		}

		OutData.Range = OutData.MaxValue - OutData.MinValue;
		OutData.HeightPerUnit = OutData.Range / FDataType::StorageRange;

		for(int32 HeightIndex = 0; HeightIndex < NumHeights; ++HeightIndex)
		{
			OutData.Heights[HeightIndex] = static_cast<typename FDataType::StorageType>((ToRealFunc(BufferView[HeightIndex]) - OutData.MinValue) / OutData.HeightPerUnit);

			int32 X = HeightIndex % (NumCols);
			int32 Y = HeightIndex / (NumCols);
			FVec3 Position(FReal(X), FReal(Y), OutData.MinValue + OutData.Heights[HeightIndex] * OutData.HeightPerUnit);
			if(HeightIndex == 0)
			{
				OutBounds = FAABB3(Position * InScale, Position * InScale);
			}
			else
			{
				OutBounds.GrowToInclude(Position * InScale);
			}
		}
		OutBounds.Thicken(UE_KINDA_SMALL_NUMBER);

		
		OutData.BuildLowResolutionData();

		if(bHaveMaterials)
		{
			if(bOnlyDefaultMaterial)
			{
				OutData.MaterialIndices.Add(0);
			}
			else
			{
				const int32 NumCells = NumHeights - NumRows - NumCols + 1;
				ensure(MaterialIndexView.Num() == NumCells);
				OutData.MaterialIndices.Empty();
				OutData.MaterialIndices.Append(MaterialIndexView.GetData(), MaterialIndexView.Num());
			}
		}
	}

	template<typename BufferType, typename ToRealFuncType>
	void EditGeomData(TArrayView<BufferType> BufferView, int32 InBeginRow, int32 InBeginCol, int32 NumRows, int32 NumCols, ToRealFuncType ToRealFunc, typename FHeightField::FDataType& OutData, FAABB3& OutBounds)
	{
		using FDataType = typename FHeightField::FDataType;

		FReal MinValue = TNumericLimits<FReal>::Max();
		FReal MaxValue = TNumericLimits<FReal>::Min();

		for(BufferType& Value : BufferView)
		{
			MinValue = FMath::Min(MinValue, ToRealFunc(Value));
			MaxValue = FMath::Max(MaxValue, ToRealFunc(Value));
		}

		const int32 EndRow = InBeginRow + NumRows;
		const int32 EndCol = InBeginCol + NumCols;

		// If our range now falls outside of the original ranges we need to resample the whole heightfield to perform the edit.
		// Here we resample everything outside of the edit and update our ranges
		const bool bNeedsResample = MinValue < OutData.MinValue || MaxValue > OutData.MaxValue;
		if(bNeedsResample)
		{
			const FReal NewMin = FMath::Min(MinValue, OutData.MinValue);
			const FReal NewMax = FMath::Max(MaxValue, OutData.MaxValue);
			const FReal NewRange = NewMax - NewMin;
			const FReal NewHeightPerUnit = NewRange / FDataType::StorageRange;

			for(int32 RowIdx = 0; RowIdx < OutData.NumRows; ++RowIdx)
			{
				for(int32 ColIdx = 0; ColIdx < OutData.NumCols; ++ColIdx)
				{
					// Provided buffer has inverted column index, invert col to ensure Heights is filled out the same was as BuildGeomData.
					const int32 HeightIndex = RowIdx * OutData.NumCols + (OutData.NumCols - 1 - ColIdx);

					if(RowIdx >= InBeginRow && RowIdx < EndRow &&
						ColIdx >= InBeginCol && ColIdx < EndCol)
					{
						// From the new set
						const int32 NewSetIndex = (RowIdx - InBeginRow) * NumCols + (ColIdx - InBeginCol);
						OutData.Heights[HeightIndex] = static_cast<typename FDataType::StorageType>((ToRealFunc(BufferView[NewSetIndex]) - NewMin) / NewHeightPerUnit);
					}
					else
					{
						// Resample existing
						const FReal ExpandedHeight = OutData.MinValue + OutData.Heights[HeightIndex] * OutData.HeightPerUnit;
						OutData.Heights[HeightIndex] = static_cast<typename FDataType::StorageType>((ExpandedHeight - NewMin) / NewHeightPerUnit);
					}

					int32 X = HeightIndex % (OutData.NumCols);
					int32 Y = HeightIndex / (OutData.NumCols);
					FVec3 Position(FReal(X), FReal(Y), NewMin + OutData.Heights[HeightIndex] * NewHeightPerUnit);
					if(HeightIndex == 0)
					{
						OutBounds = FAABB3(Position, Position);
					}
					else
					{
						OutBounds.GrowToInclude(Position);
					}
				}
			}

			OutBounds.Thicken(UE_KINDA_SMALL_NUMBER);

			OutData.MinValue = NewMin;
			OutData.MaxValue = NewMax;
			OutData.HeightPerUnit = NewHeightPerUnit;
			OutData.Range = NewRange;
		}
		else
		{
			// No resample, just push new heights into the data
			for(int32 RowIdx = InBeginRow; RowIdx < EndRow; ++RowIdx)
			{
				for(int32 ColIdx = InBeginCol; ColIdx < EndCol; ++ColIdx)
				{
					// Provided buffer has inverted column index, invert col to ensure Heights is filled out the same was as BuildGeomData.
					const int32 HeightIndex = RowIdx * OutData.NumCols + (OutData.NumCols - 1 - ColIdx);
					const int32 NewSetIndex = (RowIdx - InBeginRow) * NumCols + (ColIdx - InBeginCol);
					OutData.Heights[HeightIndex] = static_cast<typename FDataType::StorageType>((ToRealFunc(BufferView[NewSetIndex]) - OutData.MinValue) / OutData.HeightPerUnit);
				}
			}
		}

		OutData.BuildLowResolutionData();
	}

	FHeightField::FHeightField(TArray<FReal>&& Height, TArray<uint8>&& InMaterialIndices, int32 NumRows, int32 NumCols, const FVec3& InScale)
		: FImplicitObject(EImplicitObject::HasBoundingBox, ImplicitObjectType::HeightField)
	{
		auto ConversionFunc = [](const FReal InVal) -> FReal
		{
			return InVal;
		};

		BuildGeomData<FReal>(MakeArrayView(Height), MakeArrayView(InMaterialIndices), NumRows, NumCols, FVec3(1), ConversionFunc, GeomData, LocalBounds);
		CalcBounds();
		SetScale(InScale);
	}

	Chaos::FHeightField::FHeightField(TArrayView<const uint16> InHeights, TArrayView<const uint8> InMaterialIndices, int32 InNumRows, int32 InNumCols, const FVec3& InScale)
		: FImplicitObject(EImplicitObject::HasBoundingBox, ImplicitObjectType::HeightField)
	{
		auto ConversionFunc = [](const uint16 InVal) -> FReal
		{
			return (FReal)((int32)InVal - 32768);
		};

		BuildGeomData<const uint16>(InHeights, InMaterialIndices, InNumRows, InNumCols, FVec3(1), ConversionFunc, GeomData, LocalBounds);
		CalcBounds();
		SetScale(InScale);
	}

	void Chaos::FHeightField::EditHeights(TArrayView<const uint16> InHeights, int32 InBeginRow, int32 InBeginCol, int32 InNumRows, int32 InNumCols)
	{
		const int32 NumExpectedValues = InNumRows * InNumCols;
		const int32 EndRow = InBeginRow + InNumRows - 1;
		const int32 EndCol = InBeginCol + InNumCols - 1;

		if(ensure(InHeights.Num() == NumExpectedValues && InBeginRow >= 0 && InBeginCol >= 0 && EndRow < GeomData.NumRows && EndCol < GeomData.NumCols))
		{
			auto ConversionFunc = [](const uint16 InVal) -> FReal
			{
				return (FReal)((int32)InVal - 32768);
			};

			EditGeomData<const uint16>(InHeights, InBeginRow, InBeginCol, InNumRows, InNumCols, ConversionFunc, GeomData, LocalBounds);

			// Slow and dumb. TODO: Actually fix CellHeights inside EditGeomData.
			CalcBounds();
		}
	}

	void Chaos::FHeightField::EditHeights(TArrayView<FReal> InHeights, int32 InBeginRow, int32 InBeginCol, int32 InNumRows, int32 InNumCols)
	{
		const int32 NumExpectedValues = InNumRows * InNumCols;
		const int32 EndRow = InBeginRow + InNumRows - 1;
		const int32 EndCol = InBeginCol + InNumCols - 1;

		if(ensure(InHeights.Num() == NumExpectedValues && InBeginRow >= 0 && InBeginCol >= 0 && EndRow < GeomData.NumRows && EndCol < GeomData.NumCols))
		{
			auto ConversionFunc = [](const FReal InVal) -> FReal
			{
				return InVal;
			};

			EditGeomData<FReal>(InHeights, InBeginRow, InBeginCol, InNumRows, InNumCols, ConversionFunc, GeomData, LocalBounds);

			// Slow and dumb. TODO: Actually fix CellHeights inside EditGeomData.
			CalcBounds();
		}
	}

	bool Chaos::FHeightField::GetCellBounds2D(const TVec2<int32> InCoord, FBounds2D& OutBounds, const FVec2& InInflate /*= {0}*/) const
	{
		if (FlatGrid.IsValid(InCoord))
		{
			OutBounds.Min = FVec2(static_cast<FReal>(InCoord[0]), static_cast<FReal>(InCoord[1]));
			OutBounds.Max = FVec2(static_cast<FReal>(InCoord[0] + 1), static_cast<FReal>(InCoord[1] + 1));
			OutBounds.Min -= InInflate;
			OutBounds.Max += InInflate;

			return true;
		}

		return false;
	}

	FReal Chaos::FHeightField::GetHeight(int32 InIndex) const
	{
		if (CHAOS_ENSURE(InIndex >= 0 && InIndex < GeomData.Heights.Num()))
		{
			return GeomData.GetPoint(InIndex).Z;
		}

		return TNumericLimits<FReal>::Max();
	}


	FVec3 FHeightField::GetPointScaled(int32 InIndex) const
	{
		if (CHAOS_ENSURE(InIndex >= 0 && InIndex < GeomData.Heights.Num()))
		{
			return GeomData.GetPointScaled(InIndex);
		}
		
		return FVec3::ZeroVector;
	}


	FReal Chaos::FHeightField::GetHeight(int32 InX, int32 InY) const
	{
		const int32 Index = InY * GeomData.NumCols + InX;
		return GetHeight(Index);
	}

	uint8 Chaos::FHeightField::GetMaterialIndex(int32 InIndex) const
	{
		if(CHAOS_ENSURE(InIndex >= 0 && InIndex < GeomData.MaterialIndices.Num()))
		{
			return GeomData.MaterialIndices[InIndex];
		}

		return TNumericLimits<uint8>::Max();
	}

	uint8 Chaos::FHeightField::GetMaterialIndexAt(const FVec2& InGridLocationLocal) const
	{
		const FVec2 ClampledGridLocationLocal = FlatGrid.Clamp(InGridLocationLocal);
		TVec2<int32> CellCoord = FlatGrid.Cell(ClampledGridLocationLocal);

		const int32 Index = CellCoord[1] * (GeomData.NumCols - 1) + CellCoord[0];
		return GetMaterialIndex(Index);
	}

	uint8 Chaos::FHeightField::GetMaterialIndex(int32 InX, int32 InY) const
	{
		const int32 Index = InY * (GeomData.NumCols - 1) + InX;
		return GetMaterialIndex(Index);
	}

	bool Chaos::FHeightField::IsHole(int32 InIndex) const
	{
		const bool bHasMaterials = GeomData.MaterialIndices.Num() > 0;
		return bHasMaterials && GetMaterialIndex(InIndex) == TNumericLimits<uint8>::Max();
	}

	bool Chaos::FHeightField::IsHole(int32 InCellX, int32 InCellY) const
	{
		// Convert to single cell index
		const int32 Index = InCellY * (GeomData.NumCols - 1) + InCellX;
		return IsHole(Index);
	}

	struct FHeightNormalResult
	{
		Chaos::FReal Height = TNumericLimits<Chaos::FReal>::Max();
		Chaos::FVec3 Normal = Chaos::FVec3(0);
	};

	/**
	* GetHeightNormalAt always return a valid normal
	* if the point is outside of the grid, the edge is extended infinitely 
	**/
	template<bool bFillHeight, bool bFillNormal>
	FHeightNormalResult GetHeightNormalAt(const FVec2& InGridLocationLocal, const Chaos::FHeightField::FDataType& InGeomData, const TUniformGrid<Chaos::FReal, 2>& InGrid)
	{
		FHeightNormalResult Result;

		const FVec2 ClampedGridLocationLocal = InGrid.Clamp(InGridLocationLocal);
		
		TVec2<int32> CellCoord = InGrid.Cell(ClampedGridLocationLocal);
		CellCoord = InGrid.ClampIndex(CellCoord);

		const int32 SingleIndex = CellCoord[1] * (InGeomData.NumCols) + CellCoord[0];
		FVec3 Pts[4];
		InGeomData.GetPointsScaled(SingleIndex, Pts);

		const Chaos::FReal FractionX = FMath::Frac(ClampedGridLocationLocal[0]);
		const Chaos::FReal FractionY = FMath::Frac(ClampedGridLocationLocal[1]);

		if(FractionX < FractionY)
		{
			if(bFillHeight)
			{
				const static FVector Tri[3] = {FVector(0.0f, 0.0f, 0.0f), FVector(1.0f, 1.0f, 0.0f), FVector(0.0f, 1.0f, 0.0f)};
				FVector Bary = FMath::GetBaryCentric2D ({ FractionX, FractionY, 0.0f }, Tri[0], Tri[1], Tri[2]);

				Result.Height = Pts[0].Z * Bary[0] + Pts[3].Z * Bary[1] + Pts[2].Z * Bary[2];
			}

			if(bFillNormal)
			{
				const FVec3 AB = Pts[3] - Pts[0];
				const FVec3 AC = Pts[2] - Pts[0];
				Result.Normal = FVec3::CrossProduct(AB, AC).GetUnsafeNormal();
			}
		}
		else
		{
			if(bFillHeight)
			{
				const static FVector Tri[3] = {FVector(0.0f, 0.0f, 0.0f), FVector(1.0f, 0.0f, 0.0f), FVector(1.0f, 1.0f, 0.0f)};
				FVector Bary = FMath::GetBaryCentric2D({ FractionX, FractionY, 0.0f }, Tri[0], Tri[1], Tri[2]);

				Result.Height = Pts[0].Z * Bary[0] + Pts[1].Z * Bary[1] + Pts[3].Z * Bary[2];
			}

			if(bFillNormal)
			{
				const FVec3 AB = Pts[1] - Pts[0];
				const FVec3 AC = Pts[3] - Pts[0];
				Result.Normal = FVec3::CrossProduct(AB, AC).GetUnsafeNormal();
			}
		}

		return Result;
	}

	Chaos::FVec3 Chaos::FHeightField::GetNormalAt(const TVec2<FReal>& InGridLocationLocal) const
	{
		return GetHeightNormalAt<false, true>(InGridLocationLocal, GeomData, FlatGrid).Normal;

	}

	FReal Chaos::FHeightField::GetHeightAt(const TVec2<FReal>& InGridLocationLocal) const
	{
		return GetHeightNormalAt<true, false>(InGridLocationLocal, GeomData, FlatGrid).Height;
	}

	bool Chaos::FHeightField::GetCellBounds3D(const TVec2<int32> InCoord, FVec3& OutMin, FVec3& OutMax, const FVec3& InInflate /*= {0}*/) const
	{
		if (FlatGrid.IsValid(InCoord))
		{
			//todo: just compute max height, avoid extra work since this is called from tight loop
			FVec3 Min,Max;
			CalcCellBounds3D(InCoord,Min,Max);

			OutMin = FVec3(static_cast<FReal>(InCoord[0]), static_cast<FReal>(InCoord[1]), GeomData.GetMinHeight());
			OutMax = FVec3(static_cast<FReal>(InCoord[0] + 1), static_cast<FReal>(InCoord[1] + 1), Max[2]);
			OutMin = OutMin - InInflate;
			OutMax = OutMax + InInflate;

			return true;
		}

		return false;
	}

	bool Chaos::FHeightField::GetCellBounds2DScaled(const TVec2<int32> InCoord, FBounds2D& OutBounds, const FVec2& InInflate /*= {0}*/) const
	{
		if (FlatGrid.IsValid(InCoord))
		{
			OutBounds.Min = FVec2(static_cast<FReal>(InCoord[0]), static_cast<FReal>(InCoord[1]));
			OutBounds.Max = FVec2(static_cast<FReal>(InCoord[0] + 1), static_cast<FReal>(InCoord[1] + 1));
			OutBounds.Min -= InInflate;
			OutBounds.Max += InInflate;
			const FVec2 Scale2D = FVec2(GeomData.Scale[0], GeomData.Scale[1]);
			OutBounds.Min *= Scale2D;
			OutBounds.Max *= Scale2D;
			return true;
		}

		return false;
	}

	bool Chaos::FHeightField::GetCellBounds3DScaled(const TVec2<int32> InCoord, FVec3& OutMin, FVec3& OutMax, const FVec3& InInflate /*= {0}*/) const
	{
		if (FlatGrid.IsValid(InCoord))
		{
			//todo: just compute max height, avoid extra work since this is called from tight loop
			FVec3 Min,Max;
			CalcCellBounds3D(InCoord,Min,Max);

			OutMin = FVec3(static_cast<FReal>(InCoord[0]), static_cast<FReal>(InCoord[1]), GeomData.GetMinHeight());
			OutMax = FVec3(static_cast<FReal>(InCoord[0] + 1), static_cast<FReal>(InCoord[1] + 1), Max[2]);

			const FAABB3 CellBoundScaled = FAABB3::FromPoints(OutMin * GeomData.Scale, OutMax * GeomData.Scale);

			OutMin = CellBoundScaled.Min() - InInflate;
			OutMax = CellBoundScaled.Max() + InInflate;
			return true;
		}

		return false;
	}

	bool Chaos::FHeightField::CalcCellBounds3D(const TVec2<int32> InCoord, FVec3& OutMin, FVec3& OutMax, const FVec3& InInflate /*= {0}*/) const
	{
		if(FlatGrid.IsValid(InCoord))
		{
			int32 Index = InCoord[1] * (GeomData.NumCols) + InCoord[0];
			FVec3 Points[4];
			GeomData.GetPoints(Index, Points);

			const FAABB3 CellBound = FAABB3::FromPoints(Points[0], Points[1], Points[2], Points[3]);

			OutMin = CellBound.Min();
			OutMax = CellBound.Max();
			OutMin -= InInflate;
			OutMax += InInflate;

			return true;
		}

		return false;
	}

	// Raycast is navigating slowly one cell at the time, on the high resolution height field
	// This navigation is the most precise and will check if we need to compute the raycast at the triangle level
	FORCEINLINE bool FHeightField::WalkSlow(FWalkingData& WalkingData, int32 IndexLowResX, int32 IndexLowResY) const
	{
		const int32 LowResInc = WalkingData.Visitor.GeomData->LowResInc;
		while (true)
		{
			if (FlatGrid.IsValid(WalkingData.CellIdx))
			{
				PHYSICS_CSV_CUSTOM_VERY_EXPENSIVE(PhysicsCounters, NumRayHeightfieldCellVisited, 1, ECsvCustomStatOp::Accumulate);
				// Test for the cell bounding box is done in the visitor at the same time as fetching the points for the triangles
				// this avoid fetching the points twice ( here and in the visitor )

				const int32 Payload = WalkingData.CellIdx[1] * (GeomData.NumCols - 1) + WalkingData.CellIdx[0];
				const int32 SubY = (Payload) / (GeomData.NumCols - 1);
				const int32 FullIndex = Payload + SubY;
				FAABBVectorized Bounds;
				WalkingData.Visitor.GeomData->GetBoundsScaledSimd(FullIndex, Bounds);
				if (Bounds.RaycastFast(WalkingData.Visitor.StartPointSimd, WalkingData.Visitor.InvDirSimd, WalkingData.Visitor.Parallel, WalkingData.CurrentLengthSimd))
				{
					const bool bContinue = WalkingData.Visitor.VisitRaycast(WalkingData.CellIdx[1] * (GeomData.NumCols - 1) + WalkingData.CellIdx[0], WalkingData.CurrentLength);
					if (!bContinue)
					{
						return false;
					}
				}
			}

			//find next cell
			//We want to know which plane we used to cross into next cell
			const FVec2 ScaledCellCenter2D = WalkingData.ScaledMin + FVec2(static_cast<FReal>(WalkingData.CellIdx[0]) + 0.5f, static_cast<FReal>(WalkingData.CellIdx[1]) + 0.5f) * WalkingData.ScaledDx2D;
			const FVec3 ScaledCellCenter(ScaledCellCenter2D[0], ScaledCellCenter2D[1], WalkingData.ZMidPoint);

			FReal Times[3];
			FReal BestTime = WalkingData.CurrentLength;
			bool bTerminate = true;
			for (int Axis = 0; Axis < 3; ++Axis)
			{
				if (!WalkingData.bParallel[Axis])
				{
					const FReal CrossPoint = (WalkingData.Dir[Axis] * WalkingData.ScaleSign[Axis]) > 0 ? ScaledCellCenter[Axis] + WalkingData.ScaledDx[Axis] / 2 : ScaledCellCenter[Axis] - WalkingData.ScaledDx[Axis] / 2;
					const FReal Distance = CrossPoint - WalkingData.NextStart[Axis];	//note: CellCenter already has /2, we probably want to use the corner instead
					const FReal Time = Distance * WalkingData.InvDir[Axis];
					Times[Axis] = Time;
					if (Time < BestTime)
					{
						bTerminate = false;	//found at least one plane to pass through
						BestTime = Time;
					}
				}
				else
				{
					Times[Axis] = TNumericLimits<FReal>::Max();
				}
			}

			if (bTerminate)
			{
				return false;
			}

			const TVec2<int32> PrevIdx = WalkingData.CellIdx;

			for (int Axis = 0; Axis < 2; ++Axis)
			{
				WalkingData.CellIdx[Axis] += (Times[Axis] <= BestTime) ? ((WalkingData.Dir[Axis] * WalkingData.ScaleSign[Axis]) > 0 ? 1 : -1) : 0;
				if (WalkingData.CellIdx[Axis] < 0 || WalkingData.CellIdx[Axis] >= FlatGrid.Counts()[Axis])
				{
					return false;
				}
			}

			if (PrevIdx == WalkingData.CellIdx)
			{
				//crossed on z plane which means no longer in heightfield bounds
				return false;
			}

			WalkingData.NextStart = WalkingData.NextStart + WalkingData.Dir * BestTime;

			if (IndexLowResX != WalkingData.CellIdx[0] / LowResInc || IndexLowResY != WalkingData.CellIdx[1] / LowResInc)
			{
				return true;
			}

		}
	}

	// Raycast is navigating fast on the high resolution height field
	// This navigation is more precise than the low resolution walk. It would have more cache miss because it grabs several cells not necessarily contiguous in memory.
	// It navigates only on the cell that need to be checked.
	FORCEINLINE bool FHeightField::WalkFast(FWalkingData& WalkingData, const FVec2& Scale2D, const FVec3& DirScaled) const
	{
		const FReal BoundsMinZ = CachedBounds.Min().Z;
		const FReal BoundsMaxZ = CachedBounds.Max().Z;

		const int32 LowResInc = WalkingData.Visitor.GeomData->LowResInc;
		const int32 IndexLowResX = WalkingData.CellIdx[0] / LowResInc;
		const int32 IndexLowResY = WalkingData.CellIdx[1] / LowResInc;

		const TVec2<int32> NextStartInt = TVec2<int32>(static_cast<int32>(WalkingData.NextStart[0] / Scale2D[0]), static_cast<int32>(WalkingData.NextStart[1] / Scale2D[1]));
		WalkingData.CellIdx = FlatGrid.Cell(NextStartInt);

		const FVec3 NextStartOri = WalkingData.NextStart;
		const FReal Length2D = FMath::Sqrt(DirScaled[0] * DirScaled[0] + DirScaled[1] * DirScaled[1]);
		constexpr FReal FastInc2D = 2.0;  // Length in cell dimension
		const FReal FastIncScaled = FastInc2D / Length2D;

		const FVec3 InspectionStepVector = DirScaled * FastIncScaled * WalkingData.ScaledDx;
		FReal NextStartZ = FMath::Clamp(WalkingData.NextStart.Z, BoundsMinZ, BoundsMaxZ); // Numerical errors can violate this condition, so force it here.
		const FReal FastIncScaled3D = InspectionStepVector.Length();
		const FReal CellsToInspectZ = FastIncScaled3D * WalkingData.Dir.Z;
		FReal DistanceProcessed = 0.0;

		while (true)
		{
			if (!FlatGrid.IsValid(WalkingData.CellIdx))
			{
				return false;
			}

			const FVec3 NewNextStart = (InspectionStepVector + WalkingData.NextStart);
			const TVec2<int32> NewNextStartInt = TVec2<int32>(static_cast<int32>(NewNextStart[0] / Scale2D[0]), static_cast<int32>(NewNextStart[1] / Scale2D[1]));
			const TVec2<int32> NextStartInt2 = TVec2<int32>(static_cast<int32>(WalkingData.NextStart[0] / Scale2D[0]), static_cast<int32>(WalkingData.NextStart[1] / Scale2D[1]));

			const TVec2<int32> DiffInt = NewNextStartInt - NextStartInt2;
			const TVec2<int32> AddedCellIdx = TVec2<int32>(static_cast<int32>(DiffInt[0]), static_cast<int32>(DiffInt[1]));

			FAABBVectorized Bounds;
			WalkingData.Visitor.GeomData->GetBoundsScaled(WalkingData.CellIdx, AddedCellIdx, Bounds);
			if (Bounds.RaycastFast(WalkingData.Visitor.StartPointSimd, WalkingData.Visitor.InvDirSimd, WalkingData.Visitor.Parallel, WalkingData.CurrentLengthSimd))
			{
				WalkingData.NextStart = NextStartOri + WalkingData.Dir * DistanceProcessed;
				bool bContinue = WalkSlow(WalkingData, IndexLowResX, IndexLowResY);
				if (bContinue)
				{
					return true;
				}
				return false;
			}

			WalkingData.NextStart = NewNextStart;

			const TVec2<int32> PrevIdx = WalkingData.CellIdx;
			WalkingData.CellIdx = FlatGrid.Cell(NewNextStartInt);
			if (PrevIdx == WalkingData.CellIdx)
			{
				return false;
			}

			if (DistanceProcessed > WalkingData.CurrentLength || NextStartZ < BoundsMinZ || NextStartZ > BoundsMaxZ)
			{
				return false;
			}
			DistanceProcessed += FastIncScaled3D;
			NextStartZ += CellsToInspectZ;
			if (IndexLowResX != WalkingData.CellIdx[0] / LowResInc || IndexLowResY != WalkingData.CellIdx[1] / LowResInc)
			{
				// Walk on low resolution height field
				return true;
			}
		} 
	}

	// Raycast is navigating a low resolution height field
	// This navigation is less precise can have lot of false positive intersection but navigate fast with few cache miss.
	// It navigates on a squared cell number.
	FORCEINLINE bool FHeightField::WalkOnLowRes(FWalkingData& WalkingData, const FVec2& Scale2D, const FVec3& DirScaled) const
	{
		const int32 LowResInc = WalkingData.Visitor.GeomData->LowResInc;
		while (true)
		{
			if (!FlatGrid.IsValid(WalkingData.CellIdx))
			{
				return false;
			}

			FAABBVectorized Bounds;
			WalkingData.Visitor.GeomData->GetLowResBoundsScaled(WalkingData.CellIdx, Bounds);
			if (Bounds.RaycastFast(WalkingData.Visitor.StartPointSimd, WalkingData.Visitor.InvDirSimd, WalkingData.Visitor.Parallel, WalkingData.CurrentLengthSimd))
			{
				bool bContinue = WalkFast(WalkingData, Scale2D, DirScaled);
				if (bContinue)
				{
					continue; 
				}
				return false;
			}

			const FVec2 ScaledCellCenter2D = WalkingData.ScaledMin + FVec2(static_cast<FReal>(WalkingData.CellIdx[0] / LowResInc) + 0.5f, static_cast<FReal>(WalkingData.CellIdx[1] / LowResInc) + 0.5f) * (WalkingData.ScaledDx2D * (FReal)LowResInc);
			const FVec3 ScaledCellCenter(ScaledCellCenter2D[0], ScaledCellCenter2D[1], WalkingData.ZMidPoint);

			FReal Times[3];
			FReal BestTime = WalkingData.CurrentLength;
			bool bTerminate = true;
			for (int Axis = 0; Axis < 3; ++Axis)
			{
				if (!WalkingData.bParallel[Axis])
				{
					const FReal CrossPoint = (WalkingData.Dir[Axis] * WalkingData.ScaleSign[Axis]) > 0 ? ScaledCellCenter[Axis] + WalkingData.ScaledDx[Axis] * (FReal)LowResInc / 2 : ScaledCellCenter[Axis] - WalkingData.ScaledDx[Axis] * (FReal)LowResInc / 2;
					const FReal Distance = CrossPoint - WalkingData.NextStart[Axis];	//note: CellCenter already has /2, we probably want to use the corner instead
					const FReal Time = Distance * WalkingData.InvDir[Axis];
					Times[Axis] = Time;
					if (Time < BestTime)
					{
						bTerminate = false;	//found at least one plane to pass through
						BestTime = Time;
					}
				}
				else
				{
					Times[Axis] = TNumericLimits<FReal>::Max();
				}
			}

			if (bTerminate)
			{
				return false;
			}

			const TVec2<int32> PrevIdx = WalkingData.CellIdx;

			for (int Axis = 0; Axis < 2; ++Axis)
			{
				WalkingData.CellIdx[Axis] += (Times[Axis] <= BestTime) ? ((WalkingData.Dir[Axis] * WalkingData.ScaleSign[Axis]) > 0 ? LowResInc : -LowResInc) : 0;
				// Make sure CellIdx doesn't go beyond the grid
				WalkingData.CellIdx[Axis] = FMath::Min<int32>(WalkingData.CellIdx[Axis], FlatGrid.Counts()[Axis] - 1);
				WalkingData.CellIdx[Axis] = FMath::Max<int32>(WalkingData.CellIdx[Axis], 0);
			}

			if (PrevIdx == WalkingData.CellIdx)
			{
				return false;
			}
			WalkingData.NextStart = WalkingData.NextStart + WalkingData.Dir * BestTime;
		}
	}
	
	FORCEINLINE bool FHeightField::GridCast(const FVec3& StartPoint, const FVec3& Dir, const FReal Length, FHeightfieldRaycastVisitor& Visitor) const
	{
		//Is this check needed?
		if(Length < 1e-4)
		{
			return false;
		}

		FWalkingData WalkingData(Visitor);
		WalkingData.Dir = Dir;
		WalkingData.CurrentLength = Length;

		FReal InvCurrentLength = 1 / WalkingData.CurrentLength;
		for(int Axis = 0; Axis < 3; ++Axis)
		{
			WalkingData.bParallel[Axis] = FMath::IsNearlyZero(Dir[Axis], (FReal)1.e-8);
			WalkingData.InvDir[Axis] = WalkingData.bParallel[Axis] ? 0 : 1 / Dir[Axis];
		}
		
		FReal RayEntryTime;
		FReal RayExitTime;
		if(CachedBounds.RaycastFast(StartPoint, Dir, WalkingData.InvDir, WalkingData.bParallel, Length, InvCurrentLength, RayEntryTime, RayExitTime))
		{
			WalkingData.CurrentLength = RayExitTime + 1e-2; // to account for precision errors 
			WalkingData.NextStart = StartPoint + (Dir * RayEntryTime);

			const FVec2 Scale2D(GeomData.Scale[0], GeomData.Scale[1]);
			WalkingData.CellIdx = FlatGrid.Cell(TVec2<int32>(static_cast<int32>(WalkingData.NextStart[0] / Scale2D[0]), static_cast<int32>(WalkingData.NextStart[1] / Scale2D[1])));
			const FReal ZDx = CachedBounds.Extents()[2];
			WalkingData.ZMidPoint = CachedBounds.Min()[2] + ZDx * 0.5f;
			WalkingData.ScaledDx = FVec3(FlatGrid.Dx()[0] * Scale2D[0], FlatGrid.Dx()[1] * Scale2D[1], ZDx);
			WalkingData.ScaledDx2D = FVec2(WalkingData.ScaledDx[0], WalkingData.ScaledDx[1]);
			WalkingData.ScaledMin = FlatGrid.MinCorner() * Scale2D;
			WalkingData.ScaleSign = GeomData.Scale.GetSignVector();

			FVec3 DirScaled = Dir / WalkingData.ScaledDx;
			const FReal SumPlaneAxis = FMath::Abs(DirScaled[0]) + FMath::Abs(DirScaled[1]);
			const bool bCanWalk = SumPlaneAxis > UE_SMALL_NUMBER;
			WalkingData.CurrentLengthSimd = VectorSet1(static_cast<FRealSingle>(WalkingData.CurrentLength));

			if (bCanWalk)
			{
				WalkOnLowRes(WalkingData, Scale2D, DirScaled);
			}
			else 
			{
				const int32 LowResInc = Visitor.GeomData->LowResInc;
				const int32 IndexLowResX = WalkingData.CellIdx[0] / LowResInc;
				const int32 IndexLowResY = WalkingData.CellIdx[1] / LowResInc;
				const bool bContinue = WalkSlow(WalkingData, IndexLowResX, IndexLowResY);
				if (!bContinue)
				{
					return false;
				}
			}
		}
		return false;
	}

	struct F2DGridSet
	{
		F2DGridSet(TVec2<int32> Size)
			: NumX(Size[0])
			, NumY(Size[1])
		{
			int32 BitsNeeded = NumX * NumY;
			DataSize = 1 + (BitsNeeded) / 8;
			Data = MakeUnique<uint8[]>(DataSize);
			FMemory::Memzero(Data.Get(), DataSize);
		}

		bool Contains(const TVec2<int32>& Coordinate)
		{
			int32 Idx = Coordinate[1] * NumX + Coordinate[0];
			int32 ByteIdx = Idx / 8;
			int32 BitIdx = Idx % 8;
			check(ByteIdx >= 0 && ByteIdx < DataSize);
			bool bContains = (Data[ByteIdx] >> BitIdx) & 0x1;
			return bContains;
		}

		void Add(const TVec2<int32>& Coordinate)
		{
			int32 Idx = Coordinate[1] * NumX + Coordinate[0];
			int32 ByteIdx = Idx / 8;
			int32 BitIdx = Idx % 8;
			uint8 Mask = static_cast<uint8>(1 << BitIdx);
			check(ByteIdx >= 0 && ByteIdx < DataSize);
			Data[ByteIdx] |= Mask;
		}

	private:
		int32 NumX;
		int32 NumY;
		TUniquePtr<uint8[]> Data;
		int32 DataSize;
	};

	template<typename SQVisitor>
	bool FHeightField::GridSweep(const FVec3& StartPoint, const FVec3& Dir, const FReal Length, const FVec3 InHalfExtents, SQVisitor& Visitor) const
	{
		// Take the 2D portion of the extent and inflate the grid query bounds for checking against the 2D height field grid
		// to account for the thickness when querying outside but near to the edge of the grid.
		const FVec2 Inflation2D(InHalfExtents[0], InHalfExtents[1]);
		
		FBounds2D InflatedBounds = GetFlatBounds();
		InflatedBounds.Min -= Inflation2D;
		InflatedBounds.Max += Inflation2D;

		// Full extents required when querying against the actual cell geometry bounds
		const FVec3 HalfExtents3D(InHalfExtents[0], InHalfExtents[1], InHalfExtents[2]);

		const FVec3 EndPoint = StartPoint + Dir * Length;
		const FVec2 Start2D(StartPoint[0], StartPoint[1]);
		const FVec2 End2D(EndPoint[0], EndPoint[1]);
		const FVec2 Scale2D(GeomData.Scale[0], GeomData.Scale[1]);
		
		FVec2 ClippedStart;
		FVec2 ClippedEnd;

		FVec3 SignDir(0);
		SignDir[0] = FMath::Sign(Dir[0]);
		SignDir[1] = FMath::Sign(Dir[1]);
		const FVec3 InflateStartEnd = SignDir * InHalfExtents;
		const FVec3 InflateStartPoint = StartPoint - InflateStartEnd;
		const FVec3 InflateEndPoint = StartPoint + Dir * Length + InflateStartEnd;

		if (InflatedBounds.ClipLine(InflateStartPoint, InflateEndPoint, ClippedStart, ClippedEnd))
		{
			// Rasterize the line over the grid
			TVec2<int32> StartCell = FlatGrid.Cell(ClippedStart / Scale2D);
			TVec2<int32> EndCell = FlatGrid.Cell(ClippedEnd / Scale2D);
			const int32 DeltaX = FMath::Abs(EndCell[0] - StartCell[0]);
			const int32 DeltaY = -FMath::Abs(EndCell[1] - StartCell[1]);
			const bool bSameCell = DeltaX == 0 && DeltaY == 0;
			FReal CurrentLength = Length;
			if (bSameCell)
			{
				//start and end in same cell, so test it and any cells that the InHalfExtents overlap with
				FBounds2D ClippedBounds = FBounds2D::FromPoints(ClippedStart, ClippedEnd);
				ClippedBounds.Inflate(Inflation2D);

				TVec2<int32> MinCell = FlatGrid.Cell(ClippedBounds.Min / Scale2D);
				TVec2<int32> MaxCell = FlatGrid.Cell(ClippedBounds.Max / Scale2D);

				for (int32 Y = MinCell[1]; Y <= MaxCell[1]; ++Y)
				{
					for (int32 X = MinCell[0]; X <= MaxCell[0]; ++X)
					{
						bool bContinue = Visitor.VisitSweep(Y * (GeomData.NumCols - 1) + X, CurrentLength);

						if (!bContinue)
						{
							return true;
						}
					}
				}
				return false;
			}
			
			
			const int32 DirX = StartCell[0] < EndCell[0] ? 1 : -1;
			const int32 DirY = StartCell[1] < EndCell[1] ? 1 : -1;
			int32 Error = DeltaX + DeltaY;
			const TVec2<int32> ThickenDir = FMath::Abs(DeltaX) > FMath::Abs(DeltaY) ? TVec2<int32>(0, 1) : TVec2<int32>(1, 0);

			struct FQueueEntry
			{
				TVec2<int32> Index;
				FReal ToI;
			};

			// Tracking data for cells to query (similar to bounding volume approach)
			F2DGridSet Seen(FlatGrid.Counts());
			TArray<FQueueEntry> Queue;
			Queue.Add({ StartCell, -1 });
			Seen.Add(StartCell);

			// Data for fast box cast
			bool bParallel[3];
			FVec3 InvDir;

			FReal InvCurrentLength = 1 / CurrentLength;
			for (int Axis = 0; Axis < 3; ++Axis)
			{
				bParallel[Axis] = FMath::IsNearlyZero(Dir[Axis], (FReal)1.e-8);
				InvDir[Axis] = bParallel[Axis] ? 0 : 1 / Dir[Axis];
			}

			int32 QueueIndex = 0;
			while (QueueIndex < Queue.Num())
			{
				// Copy so we don't lost the entry through reallocs
				FQueueEntry CellCoord = Queue[QueueIndex++];

				if (CellCoord.ToI > CurrentLength)
				{
					continue;
				}

				{
					// Expand each cell along the thicken direction
					// Although the line should minimally thicken around the perpendicular to the line direction
					// it's cheaper to just expand in the cardinal opposite the current major direction. We end up
					// doing a broad test on more cells but avoid having to run many rasterize/walk steps for each
					// perpendicular step.
					auto Expand = [&](const TVec2<int32>& Begin, const TVec2<int32>& Direction, const int32 NumSteps)
					{
						TVec2<int32> CurrentCell = Begin;
						for (int32 CurrStep = 0; CurrStep < NumSteps; ++CurrStep)
						{
							CurrentCell += Direction;
							// Fail if we leave the grid
							if (CurrentCell[0] < 0 || CurrentCell[1] < 0 || CurrentCell[0] > FlatGrid.Counts()[0] - 1 || CurrentCell[1] > FlatGrid.Counts()[1] - 1)
							{
								break;
							}

							// No intersections here. We set the ToI to zero to cause an intersection check to happen
							// without any expansion when we reach this cell in the queue.
							if (!Seen.Contains(CurrentCell))
							{
								Seen.Add(CurrentCell);
								Queue.Add({ CurrentCell, 0 });
							}
						}
					};

					// Check the current cell, if we hit its 3D bound we can move on to narrow phase
					const TVec2<int32> Coord = CellCoord.Index;
					if (FlatGrid.IsValid(Coord))
					{
						bool bContinue = Visitor.VisitSweep(CellCoord.Index[1] * (GeomData.NumCols - 1) + CellCoord.Index[0], CurrentLength);
						if (!bContinue)
						{
							return true;
						}
					}

					// This time isn't used to reject things for this method but to flag cells that should be expanded
					if (CellCoord.ToI < 0)
					{
						// Perform expansion for thickness
						int32 ExpandAxis;
						if (ThickenDir[0] == 0)
						{
							ExpandAxis = 1;
						}
						else
						{
							ExpandAxis = 0;
						}
						FReal ExpandSize = HalfExtents3D[ExpandAxis];
						int32 Steps = FMath::TruncToInt32(FMath::RoundFromZero(ExpandSize / FMath::Abs(GeomData.Scale[ExpandAxis])));

						Expand(Coord, ThickenDir, Steps);
						Expand(Coord, -ThickenDir, Steps);

						// Walk the line and add to the queue
						if (StartCell != EndCell)
						{
							const int32 DoubleError = Error * 2;
							if (DoubleError >= DeltaY)
							{
								Error += DeltaY;
								StartCell[0] += DirX;
							}

							if (DoubleError <= DeltaX)
							{
								Error += DeltaX;
								StartCell[1] += DirY;
							}

							if (!Seen.Contains(StartCell))
							{
								Seen.Add(StartCell);
								Queue.Add({ StartCell, -1 });
							}
						}
					}
				}
			}
		}

		return false;
	}

	bool FHeightField::Raycast(const FVec3& StartPoint, const FVec3& Dir, const FReal Length, const FReal Thickness, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex) const
	{
		OutFaceIndex = INDEX_NONE;

		FHeightfieldRaycastVisitor Visitor(&GeomData, StartPoint, Dir, Thickness);

		if(Thickness > 0)
		{
			GridSweep(StartPoint, Dir, Length, FVec3(Thickness), Visitor);
		}
		else
		{
			GridCast(StartPoint, Dir, Length, Visitor);
		}

		if(Visitor.OutTime <= Length)
		{
			OutTime = Visitor.OutTime;
			OutPosition = Visitor.OutPosition;
			OutNormal = Visitor.OutNormal;
			OutFaceIndex = Visitor.OutFaceIndex;
			return true;
		}

		return false;
	}

	bool Chaos::FHeightField::GetGridIntersections(FBounds2D InFlatBounds, TArray<TVec2<int32>>& OutIntersections) const
	{
		OutIntersections.Reset();

		const FBounds2D FlatBounds = GetFlatBounds();
		FVec2 Scale2D(GeomData.Scale[0], GeomData.Scale[1]);

		InFlatBounds = FBounds2D::FromPoints(FlatBounds.Clamp(InFlatBounds.Min) / Scale2D, FlatBounds.Clamp(InFlatBounds.Max) / Scale2D);

		TVec2<int32> MinCell = FlatGrid.Cell(InFlatBounds.Min);
		TVec2<int32> MaxCell = FlatGrid.Cell(InFlatBounds.Max);

		// We want to capture the first cell (delta == 0) as well
		const int32 NumX = MaxCell[0] - MinCell[0] + 1;
		const int32 NumY = MaxCell[1] - MinCell[1] + 1;

		OutIntersections.Reserve(NumX * NumY);
		for(int32 CurrX = 0; CurrX < NumX; ++CurrX)
		{
			for(int32 CurrY = 0; CurrY < NumY; ++CurrY)
			{
				const TVec2<int32> Cell(MinCell[0] + CurrX, MinCell[1] + CurrY);
				check(FlatGrid.IsValid(Cell));
				OutIntersections.Add(Cell);
			}
		}

		return OutIntersections.Num() > 0;
	}


	bool Chaos::FHeightField::GetGridIntersectionsBatch(FBounds2D InFlatBounds, TArray<TVec2<int32>>& OutIntersections, const FAABBVectorized& Bounds) const
	{
		OutIntersections.Reset();

		const FBounds2D FlatBounds = GetFlatBounds();
		FVec2 Scale2D(GeomData.Scale[0], GeomData.Scale[1]);

		InFlatBounds = FBounds2D::FromPoints(FlatBounds.Clamp(InFlatBounds.Min) / Scale2D, FlatBounds.Clamp(InFlatBounds.Max) / Scale2D);

		TVec2<int32> MinCell = FlatGrid.Cell(InFlatBounds.Min);
		TVec2<int32> MaxCell = FlatGrid.Cell(InFlatBounds.Max);

		// We want to capture the first cell (delta == 0) as well
		const int32 MaxX = MaxCell[0] + 1;
		const int32 MaxY = MaxCell[1] + 1;

		const int32 NumX = MaxCell[0] - MaxX;
		const int32 NumY = MaxCell[1] - MaxY;
		OutIntersections.Reserve(NumX * NumY);
		
		constexpr int32 Jump = 4;
		
		FAABBVectorized CellsBounds;
		for (int32 CurrFastX = MinCell[0]; CurrFastX < MaxX; CurrFastX += Jump)
		{
			for (int32 CurrFastY = MinCell[1]; CurrFastY < MaxY; CurrFastY += Jump)
			{
				const TVec2<int32> CellIdx(CurrFastX, CurrFastY);
				const TVec2<int32> Area(FMath::Min(MaxX - CurrFastX, Jump), FMath::Min(MaxY - CurrFastY, Jump));
				GeomData.GetBoundsScaled(CellIdx, Area, CellsBounds);
				if (CellsBounds.Intersects(Bounds))
				{
					const int32 CurrMaxX = CurrFastX + Area[0];
					const int32 CurrMaxY = CurrFastY + Area[1];

					for (int32 CurrX = CurrFastX; CurrX < CurrMaxX; ++CurrX)
					{
						for (int32 CurrY = CurrFastY; CurrY < CurrMaxY; ++CurrY)
						{
							const TVec2<int32> Cell(CurrX, CurrY);
							check (FlatGrid.IsValid(Cell))
							OutIntersections.Add(Cell);
						}
					}
				}
			}
		}
		return OutIntersections.Num() > 0;
	}
	


	typename Chaos::FHeightField::FBounds2D Chaos::FHeightField::GetFlatBounds() const
	{
		FBounds2D Result;
		Result.Min = FVec2(CachedBounds.Min()[0], CachedBounds.Min()[1]);
		Result.Max = FVec2(CachedBounds.Max()[0], CachedBounds.Max()[1]);
		return Result;
	}

	bool FHeightField::Overlap(const FVec3& Point, const FReal Thickness) const
	{
		auto OverlapTriangle = [&](const FVec3& A, const FVec3& B, const FVec3& C) -> bool
		{
			const FVec3 AB = B - A;
			const FVec3 AC = C - A;
			FVec3 Normal = FVec3::CrossProduct(AB, AC);
			const FReal NormalLength = Normal.SafeNormalize();

			if(!ensure(NormalLength > UE_KINDA_SMALL_NUMBER))
			{
				return false;
			}

			const TPlane<FReal, 3> TriPlane{A, Normal};
			const FVec3 ClosestPointOnTri = FindClosestPointOnTriangle(TriPlane, A, B, C, Point);
			const FReal Distance2 = (ClosestPointOnTri - Point).SizeSquared();

			if(Distance2 <= Thickness * Thickness)	//This really only has a hope in working if thickness is > 0
			{
				return true;
			}

			return false;
		};

		FAABB3 QueryBounds(Point, Point);
		QueryBounds.Thicken(Thickness);

		FBounds2D FlatQueryBounds;
		FlatQueryBounds.Min = FVec2(QueryBounds.Min()[0], QueryBounds.Min()[1]);
		FlatQueryBounds.Max = FVec2(QueryBounds.Max()[0], QueryBounds.Max()[1]);

		TArray<TVec2<int32>> Intersections;
		FVec3 Points[4];

		GetGridIntersections(FlatQueryBounds, Intersections);

		for(const TVec2<int32>& Cell : Intersections)
		{
			const int32 SingleIndex = Cell[1] * (GeomData.NumCols) + Cell[0];
			GeomData.GetPointsScaled(SingleIndex, Points);

			if(OverlapTriangle(Points[0], Points[1], Points[3]))
			{
				return true;
			}

			if(OverlapTriangle(Points[0], Points[3], Points[2]))
			{
				return true;
			}
		}
		
		return false;
	}

	template <typename GeomType>
	bool FHeightField::GJKContactPointImp(const GeomType& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness,
		FVec3& ContactLocation, FVec3& ContactNormal, FReal& ContactPhi, int32& ContactFaceIndex) const
	{
		auto OverlapTriangle = [&](const FVec3& A, const FVec3& B, const FVec3& C,
			FVec3& LocalContactLocation, FVec3& LocalContactNormal, FReal& LocalContactPhi) -> bool
		{
			const FVec3 AB = B - A;
			const FVec3 AC = C - A;

			const FVec3 Offset = FVec3::CrossProduct(AB, AC);
			const FVec3 TriNormal = Offset.GetUnsafeNormal();
			FTriangle TriangleConvex(A, B, C);
			FTriangleRegister TriangleConvexReg(
				MakeVectorRegisterFloatFromDouble(MakeVectorRegister(A.X, A.Y, A.Z, 0.0f)),
				MakeVectorRegisterFloatFromDouble(MakeVectorRegister(B.X, B.Y, B.Z, 0.0f)),
				MakeVectorRegisterFloatFromDouble(MakeVectorRegister(C.X, C.Y, C.Z, 0.0f)));

			FReal Penetration;
			FVec3 ClosestA, ClosestB, Normal;
			int32 ClosestVertexIndexA, ClosestVertexIndexB;
			if (GJKPenetration(TriangleConvex, QueryGeom, QueryTM, Penetration, ClosestA, ClosestB, Normal, ClosestVertexIndexA, ClosestVertexIndexB, (FReal)0))
			{
				const bool bOneSidedCollision = bChaos_Collision_OneSidedHeightField;
				const bool bSkipOneSided = (bOneSidedCollision && (FVec3::DotProduct(TriNormal, Normal) < 0));
				if (!bSkipOneSided)
				{
					LocalContactLocation = ClosestB;
					LocalContactNormal = Normal; 
					LocalContactPhi = -Penetration;
					return true;
				}
			}

			return false;
		};

		bool bResult = false;
		FAABB3 QueryBounds = QueryGeom.BoundingBox();
		QueryBounds.Thicken(Thickness);
		QueryBounds = QueryBounds.TransformedAABB(QueryTM);

		FBounds2D FlatQueryBounds;
		FlatQueryBounds.Min = FVec2(QueryBounds.Min()[0], QueryBounds.Min()[1]);
		FlatQueryBounds.Max = FVec2(QueryBounds.Max()[0], QueryBounds.Max()[1]);

		TArray<TVec2<int32>> Intersections;
		FVec3 Points[4];

		GetGridIntersections(FlatQueryBounds, Intersections);

		// Forward winding point indices for each triangle
		int32 Indices[2][3] =
		{
			{0, 1, 3},
			{0, 3, 2}
		};

		// Swap indices if reverse winding
		const bool bStandardWinding = ((GeomData.Scale.X * GeomData.Scale.Y * GeomData.Scale.Z) >= FReal(0));
		if (!bStandardWinding)
		{
			Swap(Indices[0][1], Indices[0][2]);
			Swap(Indices[1][1], Indices[1][2]);
		}

		FReal LocalContactPhi = FLT_MAX;
		FVec3 LocalContactLocation, LocalContactNormal;
		for (const TVec2<int32>& Cell : Intersections)
		{
			const int32 SingleIndex = Cell[1] * GeomData.NumCols + Cell[0];
			const int32 CellIndex = Cell[1] * (GeomData.NumCols - 1) + Cell[0];
			
			// Check for holes and skip checking if we'll never collide
			if(GeomData.MaterialIndices.IsValidIndex(CellIndex) && GeomData.MaterialIndices[CellIndex] == TNumericLimits<uint8>::Max())
			{
				continue;
			}

			// The triangle is solid so proceed to test it
			GeomData.GetPointsScaled(SingleIndex, Points);

			if (OverlapTriangle(Points[Indices[0][0]], Points[Indices[0][1]], Points[Indices[0][2]], LocalContactLocation, LocalContactNormal, LocalContactPhi))
			{
				if (LocalContactPhi < ContactPhi)
				{
					ContactPhi = LocalContactPhi;
					ContactLocation = LocalContactLocation;
					ContactNormal = LocalContactNormal;
					ContactFaceIndex = CellIndex * 2; // This assumes 2 faces per cell.
				}
			}

			if (OverlapTriangle(Points[Indices[1][0]], Points[Indices[1][1]], Points[Indices[1][2]], LocalContactLocation, LocalContactNormal, LocalContactPhi))
			{
				if (LocalContactPhi < ContactPhi)
				{
					ContactPhi = LocalContactPhi;
					ContactLocation = LocalContactLocation;
					ContactNormal = LocalContactNormal;
					// This Triangle has the same faceIndex as the previous one (this is a consequence of how the materials are stored [one per cell])
					ContactFaceIndex = CellIndex * 2; // This assumes 2 faces per cell.
				}
			}
		}

		if(ContactPhi < 0)
			return true;
		return false;
	}

	bool FHeightField::GJKContactPoint(const TBox<FReal, 3>& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FVec3& ContactLocation, FVec3& ContactNormal, FReal& ContactPhi, int32& ContactFaceIndex) const
	{
		return GJKContactPointImp(QueryGeom, QueryTM, Thickness, ContactLocation, ContactNormal, ContactPhi, ContactFaceIndex);
	}

	bool FHeightField::GJKContactPoint(const FSphere& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FVec3& ContactLocation, FVec3& ContactNormal, FReal& ContactPhi, int32& ContactFaceIndex) const
	{
		return GJKContactPointImp(QueryGeom, QueryTM, Thickness, ContactLocation, ContactNormal, ContactPhi, ContactFaceIndex);
	}

	bool FHeightField::GJKContactPoint(const FCapsule& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FVec3& ContactLocation, FVec3& ContactNormal, FReal& ContactPhi, int32& ContactFaceIndex) const
	{
		return GJKContactPointImp(QueryGeom, QueryTM, Thickness, ContactLocation, ContactNormal, ContactPhi, ContactFaceIndex);
	}

	bool FHeightField::GJKContactPoint(const FConvex& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FVec3& ContactLocation, FVec3& ContactNormal, FReal& ContactPhi, int32& ContactFaceIndex) const
	{
		return GJKContactPointImp(QueryGeom, QueryTM, Thickness, ContactLocation, ContactNormal, ContactPhi, ContactFaceIndex);
	}

	bool FHeightField::GJKContactPoint(const TImplicitObjectScaled < TBox<FReal, 3>>& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FVec3& ContactLocation, FVec3& ContactNormal, FReal& ContactPhi, int32& ContactFaceIndex) const
	{
		return GJKContactPointImp(QueryGeom, QueryTM, Thickness, ContactLocation, ContactNormal, ContactPhi, ContactFaceIndex);
	}

	bool FHeightField::GJKContactPoint(const TImplicitObjectScaled < FSphere>& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FVec3& ContactLocation, FVec3& ContactNormal, FReal& ContactPhi, int32& ContactFaceIndex) const
	{
		return GJKContactPointImp(QueryGeom, QueryTM, Thickness, ContactLocation, ContactNormal, ContactPhi, ContactFaceIndex);
	}

	bool FHeightField::GJKContactPoint(const TImplicitObjectScaled < FCapsule>& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FVec3& ContactLocation, FVec3& ContactNormal, FReal& ContactPhi, int32& ContactFaceIndex) const
	{
		return GJKContactPointImp(QueryGeom, QueryTM, Thickness, ContactLocation, ContactNormal, ContactPhi, ContactFaceIndex);
	}

	bool FHeightField::GJKContactPoint(const TImplicitObjectScaled<FConvex>& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FVec3& ContactLocation, FVec3& ContactNormal, FReal& ContactPhi, int32& ContactFaceIndex) const
	{
		return GJKContactPointImp(QueryGeom, QueryTM, Thickness, ContactLocation, ContactNormal, ContactPhi, ContactFaceIndex);
	}

	template <typename QueryGeomType>
	bool FHeightField::OverlapGeomImp(const QueryGeomType& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD) const
	{
		auto OverlapTriangleMTD = [&](const FVec3& A, const FVec3& B, const FVec3& C, FMTDInfo* InnerMTD) -> bool
		{
			const FVec3 AB = B - A;
			const FVec3 AC = C - A;

			//It's most likely that the query object is in front of the triangle since queries tend to be on the outside.
			//However, maybe we should check if it's behind the triangle plane. Also, we should enforce this winding in some way
			const FVec3 Offset = FVec3::CrossProduct(AB, AC);

			FTriangle TriangleConvex(A, B, C);
			FVec3 TriangleNormal(0);
			FReal Penetration = 0;
			FVec3 ClosestA(0);
			FVec3 ClosestB(0);
			int32 ClosestVertexIndexA, ClosestVertexIndexB;
			if (GJKPenetration(TriangleConvex, QueryGeom, QueryTM, Penetration, ClosestA, ClosestB, TriangleNormal, ClosestVertexIndexA, ClosestVertexIndexB, Thickness))
			{
				// Use Deepest MTD.
				if (Penetration > InnerMTD->Penetration)
				{
					InnerMTD->Penetration = Penetration;
					InnerMTD->Normal = TriangleNormal;
					InnerMTD->Position = ClosestA;
				}
				return true;
			}

			return false;
		};

		auto OverlapTriangleNoMTD = [&](const VectorRegister4Float& A, const VectorRegister4Float& B, const VectorRegister4Float& C) -> bool
		{
			// points are assumed to be in the same space as the overlap geometry
			const VectorRegister4Float AB = VectorSubtract(B, A);
			const VectorRegister4Float AC = VectorSubtract(C, A);

			//It's most likely that the query object is in front of the triangle since queries tend to be on the outside.
			//However, maybe we should check if it's behind the triangle plane. Also, we should enforce this winding in some way
			const VectorRegister4Float Offset = VectorCross(AB, AC);

			FTriangleRegister TriangleConvex(A, B, C);
			return GJKIntersectionSameSpaceSimd(TriangleConvex, QueryGeom, FRealSingle(Thickness), VectorNormalizeSafe(Offset, GlobalVectorConstants::Float1000));
		};

		bool bResult = false;
		FAABB3 QueryBounds = QueryGeom.BoundingBox();
		QueryBounds.Thicken(Thickness);
		QueryBounds = QueryBounds.TransformedAABB(QueryTM);

		FBounds2D FlatQueryBounds;
		FlatQueryBounds.Min = FVec2(QueryBounds.Min()[0], QueryBounds.Min()[1]);
		FlatQueryBounds.Max = FVec2(QueryBounds.Max()[0], QueryBounds.Max()[1]);

		TArray<TVec2<int32>> Intersections;
		
		const FAABBVectorized QueryBoundsSimd(QueryBounds);
		if (GetGridIntersectionsBatch(FlatQueryBounds, Intersections, QueryBoundsSimd))
		{
			bool bOverlaps = false;
			if (OutMTD)
			{
				OutMTD->Normal = FVec3(0);
				OutMTD->Penetration = TNumericLimits<FReal>::Lowest();
				OutMTD->Position = FVec3(0);
				FVec3 Points[4];
				FAABB3 CellBounds;
				for (const TVec2<int32>& Cell : Intersections)
				{
					const int32 SingleIndex = Cell[1] * (GeomData.NumCols) + Cell[0];
					GeomData.GetPointsAndBoundsScaled(SingleIndex, Points, CellBounds);

					if(CellBounds.Intersects(QueryBounds) && !IsHole(Cell[0], Cell[1]))
					{
						bOverlaps |= OverlapTriangleMTD(Points[0], Points[1], Points[3], OutMTD);
						bOverlaps |= OverlapTriangleMTD(Points[0], Points[3], Points[2], OutMTD);
					}
				}
				return bOverlaps;
			}
			else
			{
				[[maybe_unused]] VectorRegister4Float X1;
				[[maybe_unused]] VectorRegister4Float X2;
				[[maybe_unused]] FRealSingle Radius;
				[[maybe_unused]] FAABBSimd AABBSimd;
				if constexpr (std::is_same<QueryGeomType, FCapsule>::value)
				{
					Radius = QueryGeom.GetRadiusf();
					const FVec3f X1f = QueryGeom.GetX1f();
					const FVec3f X2f = QueryGeom.GetX2f();
					X1 = VectorLoadFloat3(&X1f.X);
					X2 = VectorLoadFloat3(&X2f.X);
				}
				else if constexpr (std::is_same<QueryGeomType, TImplicitObjectScaled < FCapsule>>::value)
				{
					Radius = QueryGeom.GetRadiusf();
					const FVec3f X1f = QueryGeom.GetUnscaledObject()->GetX1f() * QueryGeom.GetScale();
					const FVec3f X2f = QueryGeom.GetUnscaledObject()->GetX2f() * QueryGeom.GetScale();
					X1 = VectorLoadFloat3(&X1f.X);
					X2 = VectorLoadFloat3(&X2f.X);
				}
				else if constexpr (std::is_same<QueryGeomType, Chaos::FSphere>::value)
				{
					Radius = QueryGeom.GetRadiusf();
					const FVec3f X1f = QueryGeom.GetCenterf();
					X1 = VectorLoadFloat3(&X1f.X);
				}
				else if constexpr (std::is_same<QueryGeomType, TImplicitObjectScaled < Chaos::FSphere>>::value)
				{
					Radius = FRealSingle(QueryGeom.GetRadius());
					const FVec3f X1f = QueryGeom.GetUnscaledObject()->GetCenterf() * QueryGeom.GetScale();
					X1 = VectorLoadFloat3(&X1f.X);
				}
				else if constexpr (std::is_same<QueryGeomType, TBox<FReal, 3>>::value || std::is_same<QueryGeomType, TImplicitObjectScaled<TBox<FReal, 3>>>::value)
				{
					AABBSimd = FAABBSimd(QueryGeom);
				}

				VectorRegister4Float Points[4];
				const UE::Math::TQuat<FReal>& RotationDouble = QueryTM.GetRotation();
				VectorRegister4Float Rotation = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(RotationDouble.X, RotationDouble.Y, RotationDouble.Z, RotationDouble.W));
				// Normalize rotation
				Rotation = VectorNormalizeSafe(Rotation, GlobalVectorConstants::Float0001);
				VectorRegister4Float InvRotation = VectorQuaternionInverse(Rotation);

				const UE::Math::TVector<FReal>& TranslationDouble = QueryTM.GetTranslation();
				const VectorRegister4Float Translation = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(TranslationDouble.X, TranslationDouble.Y, TranslationDouble.Z, 0.0));
				FAABBVectorized CellBounds;
				for (const TVec2<int32>& Cell : Intersections)
				{
					const int32 SingleIndex = Cell[1] * (GeomData.NumCols) + Cell[0];
					GeomData.GetPointsAndBoundsScaledSimd(SingleIndex, Points, CellBounds);
					if (CellBounds.Intersects(QueryBoundsSimd) && !IsHole(Cell[0], Cell[1]))
					{
						// pre-transform the triangle in overlap geometry space
						for (int32 Index = 0; Index < 4; ++Index)
						{
							VectorRegister4Float& P = Points[Index];
							P = VectorSubtract(P, Translation);
							P = VectorQuaternionRotateVector(InvRotation, P);
						}

						if constexpr (std::is_same<QueryGeomType, FCapsule>::value || std::is_same<QueryGeomType, TImplicitObjectScaled < FCapsule>>::value)
						{
							if (ComputeCapsuleTriangleOverlapSimd(Points[0], Points[1], Points[3], X1, X2, Radius))
							{
								return true;
							}
							if (ComputeCapsuleTriangleOverlapSimd(Points[0], Points[3], Points[2], X1, X2, Radius))
							{
								return true;
							}
						}
						else if constexpr (std::is_same<QueryGeomType, Chaos::FSphere>::value || std::is_same<QueryGeomType, TImplicitObjectScaled < Chaos::FSphere>>::value)
						{
							if (ComputeSphereTriangleOverlapSimd(Points[0], Points[1], Points[3], X1, Radius))
							{
								return true;
							}
							if (ComputeSphereTriangleOverlapSimd(Points[0], Points[3], Points[2], X1, Radius))
							{
								return true;
							}
						}
						else if constexpr (std::is_same<QueryGeomType, TBox<FReal, 3>>::value || std::is_same<QueryGeomType, TImplicitObjectScaled<TBox<FReal, 3>>>::value)
						{
							if (AABBSimd.OverlapTriangle(Points[0], Points[1], Points[3]))
							{
								return true;
							}
							if (AABBSimd.OverlapTriangle(Points[0], Points[3], Points[2]))
							{
								return true;
							}
						}
						else
						{
							if (OverlapTriangleNoMTD(Points[0], Points[1], Points[3]))
							{
								return true;
							}
							if (OverlapTriangleNoMTD(Points[0], Points[3], Points[2]))
							{
								return true;
							}
						}
					}
				}
			}
		}
		return false;
	}

	bool FHeightField::OverlapGeom(const FSphere& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD) const
	{
		return OverlapGeomImp(QueryGeom, QueryTM, Thickness, OutMTD);
	}

	bool FHeightField::OverlapGeom(const TBox<FReal, 3>& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD) const
	{
		return OverlapGeomImp(QueryGeom, QueryTM, Thickness, OutMTD);
	}

	bool FHeightField::OverlapGeom(const FCapsule& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD) const
	{
		return OverlapGeomImp(QueryGeom, QueryTM, Thickness, OutMTD);
	}

	bool FHeightField::OverlapGeom(const FConvex& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD) const
	{
		return OverlapGeomImp(QueryGeom, QueryTM, Thickness, OutMTD);
	}

	bool FHeightField::OverlapGeom(const TImplicitObjectScaled<FSphere>& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD) const
	{
		return OverlapGeomImp(QueryGeom, QueryTM, Thickness, OutMTD);
	}

	bool FHeightField::OverlapGeom(const TImplicitObjectScaled<TBox<FReal, 3>>& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD) const
	{
		return OverlapGeomImp(QueryGeom, QueryTM, Thickness, OutMTD);
	}

	bool FHeightField::OverlapGeom(const TImplicitObjectScaled<FCapsule>& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD) const
	{
		return OverlapGeomImp(QueryGeom, QueryTM, Thickness, OutMTD);
	}

	bool FHeightField::OverlapGeom(const TImplicitObjectScaled<FConvex>& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD) const
	{
		return OverlapGeomImp(QueryGeom, QueryTM, Thickness, OutMTD);
	}

	template <typename QueryGeomType>
	bool FHeightField::SweepGeomImp(const QueryGeomType& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, const FReal Thickness, bool bComputeMTD) const
	{
		bool bHit = false;
		THeightfieldSweepVisitor<QueryGeomType> SQVisitor(&GeomData, QueryGeom, StartTM, Dir, Thickness, bComputeMTD);
		const FAABB3 QueryBounds = QueryGeom.BoundingBox();
		//Need inflation to consider rotation, there's probably a cheaper way to do this
		const FVec3 RotatedExtents = StartTM.TransformVectorNoScale(QueryBounds.Extents());
		FTransform RotationTM(StartTM.GetRotation(), FVector(0));
		const FAABB3 RotatedBounds = QueryBounds.TransformedAABB(RotationTM);
		

		const FVec3 StartPoint = StartTM.TransformPositionNoScale(QueryBounds.Center());
		const FVec3 Inflation3D = RotatedBounds.Extents() * 0.5 + FVec3(Thickness);
		GridSweep(StartPoint, Dir, Length, FVec3(Inflation3D[0], Inflation3D[1], Inflation3D[2]), SQVisitor);

		if(SQVisitor.OutTime <= Length)
		{
			OutTime = SQVisitor.OutTime;
			OutPosition = SQVisitor.OutPosition;
			OutNormal = SQVisitor.OutNormal;
			OutFaceIndex = SQVisitor.OutFaceIndex;
			bHit = true;
		}

		return bHit;
	}

	bool FHeightField::SweepGeom(const FSphere& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FReal Thickness, bool bComputeMTD) const
	{
		return SweepGeomImp(QueryGeom, StartTM, Dir, Length, OutTime, OutPosition, OutNormal, OutFaceIndex, Thickness, bComputeMTD);
	}

	bool FHeightField::SweepGeom(const TBox<FReal, 3>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FReal Thickness, bool bComputeMTD) const
	{
		return SweepGeomImp(QueryGeom, StartTM, Dir, Length, OutTime, OutPosition, OutNormal, OutFaceIndex, Thickness, bComputeMTD);
	}

	bool FHeightField::SweepGeom(const FCapsule& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FReal Thickness, bool bComputeMTD) const
	{
		return SweepGeomImp(QueryGeom, StartTM, Dir, Length, OutTime, OutPosition, OutNormal, OutFaceIndex, Thickness, bComputeMTD);
	}

	bool FHeightField::SweepGeom(const FConvex& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FReal Thickness, bool bComputeMTD) const
	{
		return SweepGeomImp(QueryGeom, StartTM, Dir, Length, OutTime, OutPosition, OutNormal, OutFaceIndex, Thickness, bComputeMTD);
	}

	bool FHeightField::SweepGeom(const TImplicitObjectScaled<FSphere>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FReal Thickness, bool bComputeMTD) const
	{
		return SweepGeomImp(QueryGeom, StartTM, Dir, Length, OutTime, OutPosition, OutNormal, OutFaceIndex, Thickness, bComputeMTD);
	}

	bool FHeightField::SweepGeom(const TImplicitObjectScaled<TBox<FReal, 3>>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FReal Thickness, bool bComputeMTD) const
	{
		return SweepGeomImp(QueryGeom, StartTM, Dir, Length, OutTime, OutPosition, OutNormal, OutFaceIndex, Thickness, bComputeMTD);
	}

	bool FHeightField::SweepGeom(const TImplicitObjectScaled<FCapsule>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FReal Thickness, bool bComputeMTD) const
	{
		return SweepGeomImp(QueryGeom, StartTM, Dir, Length, OutTime, OutPosition, OutNormal, OutFaceIndex, Thickness, bComputeMTD);
	}

	bool FHeightField::SweepGeom(const TImplicitObjectScaled<FConvex>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FReal Thickness, bool bComputeMTD) const
	{
		return SweepGeomImp(QueryGeom, StartTM, Dir, Length, OutTime, OutPosition, OutNormal, OutFaceIndex, Thickness, bComputeMTD);
	}

	template <typename QueryGeomType>
	bool FHeightField::SweepGeomCCDImp(const QueryGeomType& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, const FReal InIgnorePenetration, const FReal InTargetPenetration, FReal& OutTOI, FReal& OutPhi, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex) const
	{
		// Emulate the old TargetPenetration mode which always uses the first contact encountered, as opposed to the first contact which reaches a depth of TargetPenetration
		const FReal IgnorePenetration = CVars::bCCDNewTargetDepthMode ? InIgnorePenetration : 0;
		const FReal TargetPenetration = CVars::bCCDNewTargetDepthMode ? InTargetPenetration : 0;

		const FAABB3 QueryBounds = QueryGeom.BoundingBox().TransformedAABB(FRigidTransform3(FVec3(0), StartTM.GetRotation()));

		bool bHit = false;
		THeightfieldSweepVisitorCCD<QueryGeomType> SQVisitor(&GeomData, QueryGeom, QueryBounds, StartTM, Dir, Length, IgnorePenetration, TargetPenetration);
		const FVec3 StartPoint = StartTM.TransformPositionNoScale(QueryBounds.Center());

		const FVec3 Inflation3D = QueryBounds.Extents() * FReal(0.5);
		GridSweep(StartPoint, Dir, Length, Inflation3D, SQVisitor);

		FReal TOI = (Length > 0) ? SQVisitor.OutDistance / Length : FReal(0);
		FReal Phi = SQVisitor.OutPhi;

		if (TOI <= 1)
		{
			// @todo(chaos): Legacy path to be removed when fully tested. See ComputeSweptContactTOIAndPhiAtTargetPenetration
			if (!CVars::bCCDNewTargetDepthMode)
			{
				LegacyComputeSweptContactTOIAndPhiAtTargetPenetration(FVec3::DotProduct(Dir, SQVisitor.OutNormal), Length, InIgnorePenetration, InTargetPenetration, TOI, Phi);
			}

			OutTOI = TOI;
			OutPhi = Phi;
			OutPosition = SQVisitor.OutPosition;
			OutNormal = SQVisitor.OutNormal;
			OutFaceIndex = SQVisitor.OutFaceIndex;
			bHit = true;
		}

		return bHit;
	}

	bool FHeightField::SweepGeomCCD(const FSphere& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& OutTOI, FReal& OutPhi, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal) const
	{
		return SweepGeomCCDImp(QueryGeom, StartTM, Dir, Length, IgnorePenetration, TargetPenetration, OutTOI, OutPhi, OutPosition, OutNormal, OutFaceIndex);
	}

	bool FHeightField::SweepGeomCCD(const TBox<FReal, 3>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& OutTOI, FReal& OutPhi, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal) const
	{
		return SweepGeomCCDImp(QueryGeom, StartTM, Dir, Length, IgnorePenetration, TargetPenetration, OutTOI, OutPhi, OutPosition, OutNormal, OutFaceIndex);
	}

	bool FHeightField::SweepGeomCCD(const FCapsule& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& OutTOI, FReal& OutPhi, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal) const
	{
		return SweepGeomCCDImp(QueryGeom, StartTM, Dir, Length, IgnorePenetration, TargetPenetration, OutTOI, OutPhi, OutPosition, OutNormal, OutFaceIndex);
	}

	bool FHeightField::SweepGeomCCD(const FConvex& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& OutTOI, FReal& OutPhi, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal) const
	{
		return SweepGeomCCDImp(QueryGeom, StartTM, Dir, Length, IgnorePenetration, TargetPenetration, OutTOI, OutPhi, OutPosition, OutNormal, OutFaceIndex);
	}

	bool FHeightField::SweepGeomCCD(const TImplicitObjectScaled<FSphere>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& OutTOI, FReal& OutPhi, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal) const
	{
		return SweepGeomCCDImp(QueryGeom, StartTM, Dir, Length, IgnorePenetration, TargetPenetration, OutTOI, OutPhi, OutPosition, OutNormal, OutFaceIndex);
	}

	bool FHeightField::SweepGeomCCD(const TImplicitObjectScaled<TBox<FReal, 3>>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& OutTOI, FReal& OutPhi, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal) const
	{
		return SweepGeomCCDImp(QueryGeom, StartTM, Dir, Length, IgnorePenetration, TargetPenetration, OutTOI, OutPhi, OutPosition, OutNormal, OutFaceIndex);
	}

	bool FHeightField::SweepGeomCCD(const TImplicitObjectScaled<FCapsule>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& OutTOI, FReal& OutPhi, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal) const
	{
		return SweepGeomCCDImp(QueryGeom, StartTM, Dir, Length, IgnorePenetration, TargetPenetration, OutTOI, OutPhi, OutPosition, OutNormal, OutFaceIndex);
	}

	bool FHeightField::SweepGeomCCD(const TImplicitObjectScaled<FConvex>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& OutTOI, FReal& OutPhi, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal) const
	{
		return SweepGeomCCDImp(QueryGeom, StartTM, Dir, Length, IgnorePenetration, TargetPenetration, OutTOI, OutPhi, OutPosition, OutNormal, OutFaceIndex);
	}

	int32 FHeightField::FindMostOpposingFace(const FVec3& Position, const FVec3& UnitDir, int32 HintFaceIndex, FReal SearchDist) const
	{
		const FReal SearchDist2 = SearchDist * SearchDist;

		FAABB3 QueryBounds(Position - FVec3(SearchDist), Position + FVec3(SearchDist));
		const FBounds2D FlatBounds(QueryBounds);
		TArray<TVec2<int32>> PotentialIntersections;
		GetGridIntersections(FlatBounds, PotentialIntersections);

		FReal MostOpposingDot = TNumericLimits<FReal>::Max();
		int32 MostOpposingFace = HintFaceIndex;

		auto CheckTriangle = [&](int32 FaceIndex, const FVec3& A, const FVec3& B, const FVec3& C)
		{
			const FVec3 AB = B - A;
			const FVec3 AC = C - A;
			FVec3 Normal = FVec3::CrossProduct(AB, AC);
			const FReal NormalLength = Normal.SafeNormalize();
			if(!ensure(NormalLength > UE_KINDA_SMALL_NUMBER))
			{
				//hitting degenerate triangle - should be fixed before we get to this stage
				return;
			}

			const TPlane<FReal, 3> TriPlane{A, Normal};
			const FVec3 ClosestPointOnTri = FindClosestPointOnTriangle(TriPlane, A, B, C, Position);
			const FReal Distance2 = (ClosestPointOnTri - Position).SizeSquared();
			if(Distance2 < SearchDist2)
			{
				const FReal Dot = FVec3::DotProduct(Normal, UnitDir);
				if(Dot < MostOpposingDot)
				{
					MostOpposingDot = Dot;
					MostOpposingFace = FaceIndex;
				}
			}
		};

		ensure(PotentialIntersections.Num());
		for(const TVec2<int32>& CellCoord : PotentialIntersections)
		{
			const int32 CellIndex = CellCoord[1] * (GeomData.NumCols - 1) + CellCoord[0];
			const int32 SubX = CellIndex % (GeomData.NumCols - 1);
			const int32 SubY = CellIndex / (GeomData.NumCols - 1);

			const int32 FullIndex = CellIndex + SubY;

			FVec3 Points[4];

			GeomData.GetPointsScaled(FullIndex, Points);

			CheckTriangle(CellIndex * 2, Points[0], Points[1], Points[3]);
			CheckTriangle(CellIndex * 2 + 1, Points[0], Points[3], Points[2]);
		}

		return MostOpposingFace;
	}

	FHeightField::FClosestFaceData FHeightField::FindClosestFace(const FVec3& Position, FReal SearchDist) const
	{
		FClosestFaceData Result;

		auto TestInSphere = [](const FVec3& Origin, const FReal Radius2, const FVec3& TestPosition) -> bool
		{
			return (TestPosition - Origin).SizeSquared() <= Radius2;
		};

		const FReal SearchDist2 = SearchDist * SearchDist;

		FAABB3 QueryBounds(Position - FVec3(SearchDist), Position + FVec3(SearchDist));
		const FBounds2D FlatBounds(QueryBounds);
		TArray<TVec2<int32>> PotentialIntersections;
		GetGridIntersections(FlatBounds, PotentialIntersections);

		auto CheckTriangle = [&](int32 FaceIndex, const FVec3& A, const FVec3& B, const FVec3& C)
		{
			if(TestInSphere(Position, SearchDist2, A) || TestInSphere(Position, SearchDist2, B) || TestInSphere(Position, SearchDist2, C))
			{

				const FVec3 AB = B - A;
				const FVec3 AC = C - A;
				FVec3 Normal = FVec3::CrossProduct(AB, AC);

				const FReal NormalLength = Normal.SafeNormalize();
				if(!ensure(NormalLength > UE_KINDA_SMALL_NUMBER))
				{
					//hitting degenerate triangle - should be fixed before we get to this stage
					return;
				}

				const TPlane<FReal, 3> TriPlane{ A, Normal };
				const FVec3 ClosestPointOnTri = FindClosestPointOnTriangle(TriPlane, A, B, C, Position);
				const FReal Distance2 = (ClosestPointOnTri - Position).SizeSquared();
				if(Distance2 < SearchDist2 && Distance2 < Result.DistanceToFaceSq)
				{
					Result.DistanceToFaceSq = Distance2;
					Result.FaceIndex = FaceIndex;

					const FVec3 ToTriangle = ClosestPointOnTri - Position;
					Result.bWasSampleBehind = FVec3::DotProduct(ToTriangle, Normal) > 0.0f;
				}
			}
		};

		for(const TVec2<int32> & CellCoord : PotentialIntersections)
		{
			const int32 CellIndex = CellCoord[1] * (GeomData.NumCols - 1) + CellCoord[0];
			const int32 SubX = CellIndex % (GeomData.NumCols - 1);
			const int32 SubY = CellIndex / (GeomData.NumCols - 1);

			const int32 FullIndex = CellIndex + SubY;

			FVec3 Points[4];

			GeomData.GetPointsScaled(FullIndex, Points);

			CheckTriangle(CellIndex * 2, Points[0], Points[1], Points[3]);
			CheckTriangle(CellIndex * 2 + 1, Points[0], Points[3], Points[2]);
		}

		return Result;
	}

	FVec3 FHeightField::FindGeometryOpposingNormal(const FVec3& DenormDir, int32 FaceIndex, const FVec3& OriginalNormal) const
	{
		if(ensure(FaceIndex != INDEX_NONE))
		{
			bool bSecondFace = FaceIndex % 2 != 0;

			int32 CellIndex = FaceIndex / 2;
			int32 CellY = CellIndex / (GeomData.NumCols - 1);

			FVec3 Points[4];

			GeomData.GetPointsScaled(CellIndex + CellY, Points);

			FVec3 A;
			FVec3 B;
			FVec3 C;

			if(bSecondFace)
			{
				A = Points[0];
				B = Points[3];
				C = Points[2];
			}
			else
			{
				A = Points[0];
				B = Points[1];
				C = Points[3];
			}

			const FVec3 AB = B - A;
			const FVec3 AC = C - A;
			const FVec3 ScaleSigns = GeomData.Scale.GetSignVector();
			const FReal ScaleInversion = ScaleSigns.X * ScaleSigns.Y * ScaleSigns.Z;
			FVec3 Normal = FVec3::CrossProduct(AB, AC) * ScaleInversion;
			const FReal Length = Normal.SafeNormalize();
			ensure(Length > 0);
			return Normal;
		}

		return OriginalNormal;
	}

	void FHeightField::CalcBounds()
	{
		// Flatten out the Z axis
		FlattenedBounds = GetFlatBounds();

		BuildQueryData();

		// Cache per-cell bounds
		const int32 NumX = GeomData.NumCols - 1;
		const int32 NumY = GeomData.NumRows - 1;
	}

	void FHeightField::BuildQueryData()
	{
		// NumCols and NumRows are the actual heights, there are n-1 cells between those heights
		TVec2<int32> Cells(GeomData.NumCols - 1, GeomData.NumRows - 1);
		
		FVec2 MinCorner(0, 0);
		FVec2 MaxCorner(static_cast<FReal>(GeomData.NumCols - 1), static_cast<FReal>(GeomData.NumRows - 1));
		//MaxCorner *= {GeomData.Scale[0], GeomData.Scale[1]};

		FlatGrid = TUniformGrid<FReal, 2>(MinCorner, MaxCorner, Cells);
	}

	Chaos::FReal FHeightField::PhiWithNormal(const FVec3& x, FVec3& Normal) const
	{
		Chaos::FVec2 HeightField2DPosition(x.X / GeomData.Scale.X, x.Y / GeomData.Scale.Y);

		FHeightNormalResult HeightNormal = GetHeightNormalAt<true, true>(HeightField2DPosition, GeomData, FlatGrid);
		ensure(!HeightNormal.Normal.IsZero());

		// Assume the cell below us is the correct result initially
		const Chaos::FReal& HeightAtPoint = HeightNormal.Height;
		Normal = HeightNormal.Normal;
		FReal OutPhi = x.Z - HeightAtPoint;
		
		// Need to sample all cells in a Phi radius circle on the 2D grid. Large cliffs mean that the actual
		// Phi could be in an entirely different cell.
		const FClosestFaceData ClosestFace = FindClosestFace(x, FMath::Abs(OutPhi));

		if(ClosestFace.FaceIndex > INDEX_NONE)
		{
			Normal = FindGeometryOpposingNormal(FVec3(0.0f), ClosestFace.FaceIndex, FVec3(0.0f));
			OutPhi = ClosestFace.bWasSampleBehind ? -FMath::Sqrt(ClosestFace.DistanceToFaceSq) : FMath::Sqrt(ClosestFace.DistanceToFaceSq);
		}

		return OutPhi;
	}

	bool FHeightField::GetOverlappingCellRange(const FAABB3& QueryBounds, TVec2<int32>& OutCellBegin, TVec2<int32>& OutCellEnd) const
	{
		// Calculate the square cell range that overlaps the input query bounds (which is in local space)
		const FBounds2D HeightfieldFlatBounds = GetFlatBounds();
		FVec2 Scale2D(GeomData.Scale[0], GeomData.Scale[1]);
		FBounds2D GridQueryBounds;
		GridQueryBounds.Min = FVec2(QueryBounds.Min()[0], QueryBounds.Min()[1]);
		GridQueryBounds.Max = FVec2(QueryBounds.Max()[0], QueryBounds.Max()[1]);
		GridQueryBounds = FBounds2D::FromPoints(HeightfieldFlatBounds.Clamp(GridQueryBounds.Min) / Scale2D, HeightfieldFlatBounds.Clamp(GridQueryBounds.Max) / Scale2D);
		OutCellBegin = FlatGrid.Cell(GridQueryBounds.Min);
		OutCellEnd = FlatGrid.Cell(GridQueryBounds.Max);
		OutCellEnd[0] += 1;
		OutCellEnd[1] += 1;

		// @todo(chaos): this should return false if QueryBounds does not overlap the heightfield
		return true;
	}

	void FHeightField::CollectTriangles(const FAABB3& QueryBounds, const FRigidTransform3& QueryTransform, const FAABB3& ObjectBounds, Private::FMeshContactGenerator& Collector) const
	{
		TVec2<int32> CellBegin, CellEnd;
		if (!GetOverlappingCellRange(QueryBounds, CellBegin, CellEnd))
		{
			return;
		}

		// Tell the collector how many triangles to expect
		const int32 NumCells = (CellEnd[0] - CellBegin[0]) * (CellEnd[1] - CellBegin[1]);
		Collector.BeginCollect(2 * NumCells);

		FVec3 Points[4];
		FTriangle Triangles[2];
		const bool bStandardWinding = ((GeomData.Scale.X * GeomData.Scale.Y * GeomData.Scale.Z) >= FReal(0));

		// Visit all the cells in stored order
		for (int32 CellX = CellBegin[0]; CellX < CellEnd[0]; ++CellX)
		{
			for (int32 CellY = CellBegin[1]; CellY < CellEnd[1]; ++CellY)
			{
				const int32 CellVertexIndex = CellY * GeomData.NumCols + CellX;	// First vertex in cell
				const int32 CellIndex = CellY * (GeomData.NumCols - 1) + CellX;

				// Check for holes and skip checking if we'll never collide
				if (GeomData.MaterialIndices.IsValidIndex(CellIndex) && GeomData.MaterialIndices[CellIndex] == TNumericLimits<uint8>::Max())
				{
					continue;
				}

				// The triangle is solid so proceed to test it
				FAABB3 CellBounds;
				GeomData.GetPointsAndBoundsScaled(CellVertexIndex, Points, CellBounds);

				// Perform a 3D bounds check now in local space
				if (CellBounds.Intersects(QueryBounds))
				{
					// Transform points into the space of the query
					// @todo(chaos): duplicate work here when we overlap lots of cells. We could generate the transformed verts for the full query once...
					Points[0] = QueryTransform.TransformPositionNoScale(Points[0]);
					Points[1] = QueryTransform.TransformPositionNoScale(Points[1]);
					Points[2] = QueryTransform.TransformPositionNoScale(Points[2]);
					Points[3] = QueryTransform.TransformPositionNoScale(Points[3]);

					// Perform a 3D bounds check in object space
					FAABB3 ObjectCellBounds = FAABB3::FromPoints(Points[0], Points[1], Points[2], Points[3]);

					if (ObjectBounds.Intersects(ObjectCellBounds))
					{
						// @todo(chaos): utility function for this
						const int32 VertexIndex0 = CellVertexIndex;
						const int32 VertexIndex1 = CellVertexIndex + 1;
						const int32 VertexIndex2 = CellVertexIndex + GeomData.NumCols;
						const int32 VertexIndex3 = CellVertexIndex + GeomData.NumCols + 1;

						const int32 FaceIndex0 = CellIndex * 2 + 0;
						const int32 FaceIndex1 = CellIndex * 2 + 1;

						if (bStandardWinding)
						{
							Triangles[0] = FTriangle(Points[0], Points[1], Points[3]);
							Triangles[1] = FTriangle(Points[0], Points[3], Points[2]);
							Collector.AddTriangle(Triangles[0], FaceIndex0, VertexIndex0, VertexIndex1, VertexIndex3);
							Collector.AddTriangle(Triangles[1], FaceIndex1, VertexIndex0, VertexIndex3, VertexIndex2);
						}
						else
						{
							Triangles[0] = FTriangle(Points[0], Points[3], Points[1]);
							Triangles[1] = FTriangle(Points[0], Points[2], Points[3]);
							Collector.AddTriangle(Triangles[0], FaceIndex0, VertexIndex0, VertexIndex3, VertexIndex1);
							Collector.AddTriangle(Triangles[1], FaceIndex1, VertexIndex0, VertexIndex2, VertexIndex3);
						}
					}
				}
			}
		}

		Collector.EndCollect();
	}

}
