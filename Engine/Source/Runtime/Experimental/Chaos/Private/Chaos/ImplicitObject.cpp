// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/ImplicitObject.h"
#include "Chaos/BVHParticles.h"
#include "Chaos/Box.h"
#include "Chaos/Capsule.h"
#include "Chaos/Convex.h"
#include "Chaos/HeightField.h"
#include "Chaos/ImplicitObjectScaled.h"
#include "Chaos/ImplicitObjectTransformed.h"
#include "Chaos/ImplicitObjectUnion.h"
#include "Chaos/Levelset.h"
#include "Chaos/Plane.h"
#include "Chaos/Sphere.h"
#include "Chaos/TaperedCylinder.h"
#include "Chaos/TaperedCapsule.h"
#include "Chaos/TrackedGeometryManager.h"
#include "Chaos/TriangleMeshImplicitObject.h"
#include "HAL/IConsoleManager.h"
#include "UObject/DestructionObjectVersion.h"
#include "UObject/ReleaseObjectVersion.h"
#include "Chaos/MLLevelset.h" 

namespace Chaos
{
	FImplicitObject::FImplicitObject(int32 Flags, EImplicitObjectType InType)
		: Margin(0.0f)
		, bIsConvex(!!(Flags& EImplicitObject::IsConvex))
		, bDoCollide(!(Flags& EImplicitObject::DisableCollisions))
		, bHasBoundingBox(!!(Flags& EImplicitObject::HasBoundingBox))
#if TRACK_CHAOS_GEOMETRY
		, bIsTracked(false)
#endif
		, Type(InType)
		, CollisionType(InType)
	{
	}

	FImplicitObject::~FImplicitObject()
	{
		// CVD caches geometry hashes using ptrs as key. There is a chance that this ptr could get re-used
		CVD_TRACE_INVALIDATE_CACHED_GEOMETRY(this)

#if TRACK_CHAOS_GEOMETRY
		if(bIsTracked)
		{
			FTrackedGeometryManager::Get().RemoveGeometry(this);
		}
#endif
	}

#if TRACK_CHAOS_GEOMETRY

	void FImplicitObject::Track(TSerializablePtr<FImplicitObject> This, const FString& DebugInfo)
	{
		if(ensure(This.Get() == this))
		{
			if(!bIsTracked)
			{
				bIsTracked = true;
				FTrackedGeometryManager::Get().AddGeometry(This, DebugInfo);
			}
		}
	}

#endif

	EImplicitObjectType FImplicitObject::GetType() const
	{
		return Type;
	}

	EImplicitObjectType FImplicitObject::GetCollisionType() const
	{
		return CollisionType;
	}

	bool FImplicitObject::IsValidGeometry() const
	{
		return true;
	}

	Chaos::FImplicitObjectPtr FImplicitObject::CopyGeometry() const
	{
		check(false);
		return nullptr;
	}

	Chaos::FImplicitObjectPtr FImplicitObject::CopyGeometryWithScale(const FVec3& Scale) const
	{
		check(false);
		return nullptr;
	}

	bool FImplicitObject::IsUnderlyingUnion() const
	{
		return (Type == ImplicitObjectType::Union) || (Type == ImplicitObjectType::UnionClustered);
	}

	bool FImplicitObject::IsUnderlyingMesh() const
	{
		const EImplicitObjectType InnerType = GetInnerType(Type);
		return (InnerType == ImplicitObjectType::TriangleMesh) || (InnerType == ImplicitObjectType::HeightField);
	}

	FReal FImplicitObject::SignedDistance(const FVec3& x) const
	{
		FVec3 Normal;
		return PhiWithNormal(x, Normal);
	}

	FVec3 FImplicitObject::Normal(const FVec3& x) const
	{
		FVec3 Normal;
		PhiWithNormal(x, Normal);
		return Normal;
	}

	const FAABB3 FImplicitObject::BoundingBox() const
	{
		check(false);
		static const FAABB3 Unbounded(FVec3(-FLT_MAX), FVec3(FLT_MAX));
		return Unbounded;
	}

	// @todo(mlentine): This is a lot of duplication from the collisions code that should be reduced
	Pair<FVec3, bool> FImplicitObject::FindDeepestIntersection(const FImplicitObject* Other, const FBVHParticles* Particles, const FMatrix33& OtherToLocalTransform, const FReal Thickness) const
	{
		// Do analytics
		// @todo(mlentine): Should we do a convex pass here?
		if(!Particles)
		{
			return MakePair(FVec3(0), false);
		}
		FVec3 Point;
		FReal Phi = Thickness;
		if(HasBoundingBox())
		{
			FAABB3 ImplicitBox = BoundingBox().TransformedAABB((FMatrix)OtherToLocalTransform.Inverse());
			ImplicitBox.Thicken(Thickness);
			TArray<int32> PotentialParticles = Particles->FindAllIntersections(ImplicitBox);
			for(int32 i : PotentialParticles)
			{
				FVec3 LocalPoint = OtherToLocalTransform.TransformPosition(Particles->GetX(i));
				FReal LocalPhi = SignedDistance(LocalPoint);
				if(LocalPhi < Phi)
				{
					Phi = LocalPhi;
					Point = Particles->GetX(i);
				}
			}
		}
		else
		{
			return FindDeepestIntersection(Other, static_cast<const FParticles*>(Particles), OtherToLocalTransform, Thickness);
		}
		return MakePair(Point, Phi < Thickness);
	}

	Pair<FVec3, bool> FImplicitObject::FindDeepestIntersection(const FImplicitObject* Other, const FParticles* Particles, const FMatrix33& OtherToLocalTransform, const FReal Thickness) const
	{
		// Do analytics
		// @todo(mlentine): Should we do a convex pass here?
		if(!Particles)
		{
			return MakePair(FVec3(0), false);
		}
		FVec3 Point;
		FReal Phi = Thickness;
		int32 NumParticles = Particles->Size();
		for(int32 i = 0; i < NumParticles; ++i)
		{
			FVec3 LocalPoint = OtherToLocalTransform.TransformPosition(Particles->GetX(i));
			FReal LocalPhi = SignedDistance(LocalPoint);
			if(LocalPhi < Phi)
			{
				Phi = LocalPhi;
				Point = Particles->GetX(i);
			}
		}
		return MakePair(Point, Phi < Thickness);
	}

	Pair<FVec3, bool> FImplicitObject::FindClosestIntersection(const FVec3& StartPoint, const FVec3& EndPoint, const FReal Thickness) const
	{
		constexpr FReal Epsilon = (FReal)1e-4;
		constexpr FReal EpsilonSquared = Epsilon * Epsilon;

		//Consider 0 thickness with Start sitting on abs(Phi) < Epsilon. This is a common case; for example a particle sitting perfectly on a floor. In this case intersection could return false.
		//If start is in this fuzzy region we simply return that spot snapped onto the surface. This is valid because low precision means we don't really know where we are, so let's take the cheapest option
		//If end is in this fuzzy region it is also a valid hit. However, there could be multiple hits between start and end and since we want the first one, we can't simply return this point.
		//As such we move end away from start (and out of the fuzzy region) so that we always get a valid intersection if no earlier ones exist
		//When Thickness > 0 the same idea applies, but we must consider Phi = (Thickness - Epsilon, Thickness + Epsilon)
		FVec3 Normal;
		const FReal Phi = PhiWithNormal(StartPoint, Normal);
		if(FMath::IsNearlyEqual(Phi, Thickness, Epsilon))
		{
			return MakePair(FVec3(StartPoint - Normal * Phi), true); //snap to surface
		}

		FVec3 ModifiedEnd = EndPoint;
		{
			const FVec3 OriginalStartToEnd = (EndPoint - StartPoint);
			const FReal OriginalLength2 = OriginalStartToEnd.SizeSquared();
			if(OriginalLength2 < EpsilonSquared)
			{
				return MakePair(FVec3(0), false); //start was not close to surface, and end is very close to start so no hit
			}

			FVec3 EndNormal;
			const FReal EndPhi = PhiWithNormal(EndPoint, EndNormal);
			if(FMath::IsNearlyEqual(EndPhi, Thickness, Epsilon))
			{
				//We want to push End out of the fuzzy region. Moving along the normal direction is best since direction could be nearly parallel with fuzzy band
				//To ensure an intersection, we must go along the normal, but in the same general direction as the ray.
				const FVec3 OriginalDir = OriginalStartToEnd / FMath::Sqrt(OriginalLength2);
				const FReal Dot = FVec3::DotProduct(OriginalDir, EndNormal);
				if(FMath::IsNearlyZero(Dot, Epsilon))
				{
					//End is in the fuzzy region, and the direction from start to end is nearly parallel with this fuzzy band, so we should just return End since no other hits will occur
					return MakePair(FVec3(EndPoint - Normal * Phi), true); //snap to surface
				}
				else
				{
					ModifiedEnd = EndPoint + 2.f * Epsilon * FMath::Sign(Dot) * EndNormal; //get out of fuzzy region
				}
			}
		}

		return FindClosestIntersectionImp(StartPoint, ModifiedEnd, Thickness);
	}

	FRealSingle ClosestIntersectionStepSizeMultiplier = 0.5f;
	FAutoConsoleVariableRef CVarClosestIntersectionStepSizeMultiplier(TEXT("p.ClosestIntersectionStepSizeMultiplier"), ClosestIntersectionStepSizeMultiplier, TEXT("When raycasting we use this multiplier to substep the travel distance along the ray. Smaller number gives better accuracy at higher cost"));

	Pair<FVec3, bool> FImplicitObject::FindClosestIntersectionImp(const FVec3& StartPoint, const FVec3& EndPoint, const FReal Thickness) const
	{
		FReal Epsilon = (FReal)1e-4;

		FVec3 Ray = EndPoint - StartPoint;
		FReal Length = Ray.Size();
		FVec3 Direction = Ray.GetUnsafeNormal(); //this is safe because StartPoint and EndPoint were already tested to be far enough away. In the case where ModifiedEnd is pushed, we push it along the direction so it can only get farther
		FVec3 EndNormal;
		const FReal EndPhi = PhiWithNormal(EndPoint, EndNormal);
		FVec3 ClosestPoint = StartPoint;

		FVec3 Normal;
		FReal Phi = PhiWithNormal(ClosestPoint, Normal);

		while(Phi > Thickness + Epsilon)
		{
			ClosestPoint += Direction * (Phi - Thickness) * (FReal)ClosestIntersectionStepSizeMultiplier;
			if((ClosestPoint - StartPoint).Size() > Length)
			{
				if(EndPhi < Thickness + Epsilon)
				{
					return MakePair(FVec3(EndPoint + EndNormal * (-EndPhi + Thickness)), true);
				}
				return MakePair(FVec3(0), false);
			}
			// If the Change is too small we want to nudge it forward. This makes it possible to miss intersections very close to the surface but is more efficient and shouldn't matter much.
			if((Phi - Thickness) < (FReal)1e-2)
			{
				ClosestPoint += Direction * (FReal)1e-2;
				if((ClosestPoint - StartPoint).Size() > Length)
				{
					if(EndPhi < Thickness + Epsilon)
					{
						return MakePair(FVec3(EndPoint + EndNormal * (-EndPhi + Thickness)), true);
					}
					else
					{
						return MakePair(FVec3(0), false);
					}
				}
			}
			FReal NewPhi = PhiWithNormal(ClosestPoint, Normal);
			if(NewPhi >= Phi)
			{
				if(EndPhi < Thickness + Epsilon)
				{
					return MakePair(FVec3(EndPoint + EndNormal * (-EndPhi + Thickness)), true);
				}
				return MakePair(FVec3(0), false);
			}
			Phi = NewPhi;
		}
		if(Phi < Thickness + Epsilon)
		{
			ClosestPoint += Normal * (-Phi + Thickness);
		}
		return MakePair(ClosestPoint, true);
	}

	void FImplicitObject::FindAllIntersectingObjects(TArray<Pair<const FImplicitObject*, FRigidTransform3>>& Out, const FAABB3& LocalBounds) const
	{
		if(!HasBoundingBox() || LocalBounds.Intersects(BoundingBox()))
		{
			Out.Add(MakePair(this, FRigidTransform3(FVec3(0), FRotation3::FromElements(FVec3(0), (FReal)1))));
		}
	}

	FArchive& FImplicitObject::SerializeLegacyHelper(FArchive& Ar, TUniquePtr<FImplicitObject>& Value)
	{
		bool bExists = Value.Get() != nullptr;
		Ar << bExists;
		if(bExists)
		{
			if(Ar.IsLoading())
			{
				uint8 ObjectType;
				Ar << ObjectType;
				switch((EImplicitObjectType)ObjectType)
				{
				case ImplicitObjectType::Sphere: { Value = TUniquePtr<FSphere>(new FSphere()); break; }
				case ImplicitObjectType::Box: { Value = TUniquePtr<TBox<FReal, 3>>(new TBox<FReal, 3>()); break; }
				case ImplicitObjectType::Plane: { Value = TUniquePtr<TPlane<FReal, 3>>(new TPlane<FReal, 3>()); break; }
				case ImplicitObjectType::LevelSet: { Value = TUniquePtr<FLevelSet>(new FLevelSet()); break; }
				default: check(false);
				}
			}
			else
			{
				if(Value->Type == ImplicitObjectType::Sphere || Value->Type == ImplicitObjectType::Box || Value->Type == ImplicitObjectType::Plane || Value->Type == ImplicitObjectType::LevelSet)
				{
					Ar << Value->Type;
				}
				else
				{
					check(false); //should not be serializing this out
				}
			}
			Ar << *Value;
		}
		return Ar;
	}

	void FImplicitObject::SerializeImp(FArchive& Ar)
	{
		Ar.UsingCustomVersion(FDestructionObjectVersion::GUID);
		if(Ar.CustomVer(FDestructionObjectVersion::GUID) >= FDestructionObjectVersion::ChaosArchiveAdded)
		{
			Ar << bIsConvex << bDoCollide;
		}

		if(Ar.CustomVer(FDestructionObjectVersion::GUID) <= FDestructionObjectVersion::ImplicitObjectDoCollideAttribute)
		{
			bDoCollide = true;
		}

		Ar.UsingCustomVersion(FReleaseObjectVersion::GUID);
		if(Ar.CustomVer(FReleaseObjectVersion::GUID) > FReleaseObjectVersion::CustomImplicitCollisionType)
		{
			Ar << CollisionType;
		}
		else
		{
			CollisionType = Type;
		}

		// NOTE: Do not serialize Margin in FImplicitObject base class - this is handled by derived types that use it
	}

	void FImplicitObject::Serialize(FChaosArchive& Ar)
	{
		SerializeImp(Ar);
	}

	const FName FImplicitObject::GetTypeName(const EImplicitObjectType InType)
	{
		static const FName SphereName = "Sphere";
		static const FName BoxName = "Box";
		static const FName PlaneName = "Plane";
		static const FName CapsuleName = "Capsule";
		static const FName TransformedName = "Transformed";
		static const FName UnionName = "Union";
		static const FName LevelSetName = "LevelSet";
		static const FName MLLevelSetName = "MLLevelSet";
		static const FName UnknownName = "Unknown";
		static const FName ConvexName = "Convex";
		static const FName TaperedCylinderName = "TaperedCylinder";
		static const FName CylinderName = "Cylinder";
		static const FName TriangleMeshName = "TriangleMesh";
		static const FName HeightFieldName = "HeightField";
		static const FName TaperedCapsuleName = "TaperedCapsule";
		static const FName UnionClusteredName = "UnionClustered";

		switch(GetInnerType(InType))
		{
		case ImplicitObjectType::Sphere: return SphereName;
		case ImplicitObjectType::Box: return BoxName;
		case ImplicitObjectType::Plane: return PlaneName;
		case ImplicitObjectType::Capsule: return CapsuleName;
		case ImplicitObjectType::Transformed: return TransformedName;
		case ImplicitObjectType::Union: return UnionName;
		case ImplicitObjectType::LevelSet: return LevelSetName;
		case ImplicitObjectType::MLLevelSet: return MLLevelSetName;
		case ImplicitObjectType::Unknown: return UnknownName;
		case ImplicitObjectType::Convex: return ConvexName;
		case ImplicitObjectType::TaperedCylinder: return TaperedCylinderName;
		case ImplicitObjectType::Cylinder: return CylinderName;
		case ImplicitObjectType::TriangleMesh: return TriangleMeshName;
		case ImplicitObjectType::HeightField: return HeightFieldName;
		case ImplicitObjectType::TaperedCapsule: return TaperedCapsuleName;
		case ImplicitObjectType::UnionClustered: return UnionClusteredName;
		}
		return NAME_None;
	}

	FImplicitObject* FImplicitObject::SerializationFactory(FChaosArchive& Ar, FImplicitObject* Obj)
	{
		int8 ObjectType = Ar.IsLoading() ? 0 : (int8)Obj->Type;
		Ar << ObjectType;

		Ar.UsingCustomVersion(FExternalPhysicsCustomObjectVersion::GUID);
		if(Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) >= FExternalPhysicsCustomObjectVersion::ScaledGeometryIsConcrete)
		{
			if(IsScaled(ObjectType))
			{
				EImplicitObjectType InnerType = GetInnerType(ObjectType);
				switch(InnerType)
				{
				case ImplicitObjectType::Convex: if(Ar.IsLoading()){ return new TImplicitObjectScaled<FConvex>(); } break;
				case ImplicitObjectType::TriangleMesh: if(Ar.IsLoading()){ return new TImplicitObjectScaled<FTriangleMeshImplicitObject>(); } break;
				default: check(false);
				}

				return nullptr;
			}
		}

		if(IsInstanced(ObjectType))
		{
			EImplicitObjectType InnerType = GetInnerType(ObjectType);
			switch(InnerType)
			{
			case ImplicitObjectType::Convex: if(Ar.IsLoading()) { return new TImplicitObjectInstanced<FConvex>(); } break;
			case ImplicitObjectType::TriangleMesh: if(Ar.IsLoading()) { return new TImplicitObjectInstanced<FTriangleMeshImplicitObject>(); } break;
			default: check(false);
			}

			return nullptr;
		}

		switch((EImplicitObjectType)ObjectType)
		{
		case ImplicitObjectType::Sphere: if(Ar.IsLoading()) { return new FSphere(); } break;
		case ImplicitObjectType::Box: if(Ar.IsLoading()) { return new TBox<FReal, 3>(); } break;
		case ImplicitObjectType::Plane: if(Ar.IsLoading()) { return new TPlane<FReal, 3>(); } break;
		case ImplicitObjectType::Capsule: if(Ar.IsLoading()) { return new FCapsule(); } break;
		case ImplicitObjectType::Transformed: if(Ar.IsLoading()) { return new TImplicitObjectTransformed<FReal, 3>(); } break;
		case ImplicitObjectType::Union: if(Ar.IsLoading()) { return new FImplicitObjectUnion(); } break;
		case ImplicitObjectType::UnionClustered: if(Ar.IsLoading()) { return new FImplicitObjectUnionClustered(); } break;
		case ImplicitObjectType::LevelSet: if(Ar.IsLoading()) { return new FLevelSet(); } break;
		case ImplicitObjectType::Convex: if(Ar.IsLoading()) { return new FConvex(); } break;
		case ImplicitObjectType::TaperedCylinder: if(Ar.IsLoading()) { return new FTaperedCylinder(); } break;
		case ImplicitObjectType::TaperedCapsule: if(Ar.IsLoading()) { return new FTaperedCapsule(); } break;
		case ImplicitObjectType::TriangleMesh: if(Ar.IsLoading()) { return new FTriangleMeshImplicitObject(); } break;
		case ImplicitObjectType::DEPRECATED_Scaled:
		{
			ensure(Ar.IsLoading() && (Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) < FExternalPhysicsCustomObjectVersion::ScaledGeometryIsConcrete));
			return new TImplicitObjectScaledGeneric<FReal, 3>();
		}
		case ImplicitObjectType::HeightField: if(Ar.IsLoading()) { return new FHeightField(); } break;
		case ImplicitObjectType::Cylinder: if(Ar.IsLoading()) { return new FCylinder(); } break;
		default:
			check(false);
		}
		return nullptr;
	}
}
