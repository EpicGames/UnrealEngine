// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/ImplicitObjectBVH.h"
#include "Chaos/ChaosArchive.h"
#include "Misc/MemStack.h"

namespace Chaos
{
	namespace CVars
	{
		extern FRealSingle ChaosUnionBVHSplitBias;

		extern bool bChaosImplicitBVHOptimizedCountLeafObjects;
	}

	namespace Private
	{
		static_assert(sizeof(FImplicitBVHObject) == 72, "FImplicitBVHObject was packed to avoid any padding in it");

		FImplicitBVHObject::FImplicitBVHObject()
		{
		}

		FImplicitBVHObject::FImplicitBVHObject(
			const TSerializablePtr<FImplicitObject>& InGeometry, 
			const FVec3& InX, 
			const FRotation3& InR, 
			const FAABB3& InBounds,
			const int32 InRootObjectIndex,
			const int32 InObjectIndex)
			: R{ (float)InR.X, (float)InR.Y, (float)InR.Z, (float)InR.W }
			, X(FVec3f(InX))
			, Bounds(FAABB3f(InBounds))
			, RootObjectIndex(InRootObjectIndex)
			, Geometry(InGeometry)
			, ObjectIndex(InObjectIndex)
		{
		}

		///////////////////////////////////////////////////////////////////////////////////////////
		///////////////////////////////////////////////////////////////////////////////////////////
		///////////////////////////////////////////////////////////////////////////////////////////
		///////////////////////////////////////////////////////////////////////////////////////////

		int32 FImplicitBVH::CountLeafObjects(const TArrayView<const Chaos::FImplicitObjectPtr>& InRootObjects)
		{
			// Count the objects in the hierarchy
			int32 NumObjects = 0;
			if (CVars::bChaosImplicitBVHOptimizedCountLeafObjects)
			{
				for (const Chaos::FImplicitObjectPtr& RootObject : InRootObjects)
				{
					NumObjects += RootObject->CountLeafObjectsInHierarchy();
				}
			}
			else
			{
				for (const Chaos::FImplicitObjectPtr& RootObject : InRootObjects)
				{
					RootObject->VisitLeafObjects(
						[&NumObjects](const FImplicitObject* Object, const FRigidTransform3& ParentTransform, const int32 RootObjectIndex, const int32 ObjectIndex, const int32 LeafObjectIndex)
						{ 
							++NumObjects;
						});
				}
			}

			return NumObjects;
		}
		
		void FImplicitBVH::CollectLeafObject(const FImplicitObject* Object, 
			const FRigidTransform3& ParentTransform, const int32 RootObjectIndex, TArray<FImplicitBVHObject>& LeafObjects, const int32 LeafObjectIndex)
		{
			// @todo(chaos): clean this up (SetFromRawLowLevel). We know all the objects we visit are children of a UniquePtr because we own it
			TSerializablePtr<FImplicitObject> SerializableObject;
			SerializableObject.SetFromRawLowLevel(Object);

			LeafObjects.Emplace(
				SerializableObject,
				ParentTransform.GetTranslation(),
				ParentTransform.GetRotation(),
				Object->CalculateTransformedBounds(ParentTransform),
				RootObjectIndex,
				LeafObjectIndex);
		}

		FImplicitBVH::FObjects FImplicitBVH::CollectLeafObjects(const TArrayView<const Chaos::FImplicitObjectPtr>& InRootObjects)
		{
			// We visit the hierarchy once to ensure we can create a tight-fitting array of leaf objects (the array growth
			// policy will over-allocate if we don't size exactly)
			TArray<FImplicitBVHObject> Objects;
			Objects.Reserve(CountLeafObjects(InRootObjects));

			for (int32 RootObjectIndex = 0; RootObjectIndex < InRootObjects.Num(); ++RootObjectIndex)
			{
				InRootObjects[RootObjectIndex]->VisitLeafObjects(
				[RootObjectIndex, &Objects](const FImplicitObject* Object, const FRigidTransform3& ParentTransform,
					const int32 UnusedRootObjectIndex, const int32 UnusedObjectIndex, const int32 UnusedLeafObjectIndex)
				{
					CollectLeafObject(Object, ParentTransform, RootObjectIndex, Objects, Objects.Num());
				});
			}

			return Objects;
		}

		TUniquePtr<FImplicitBVH> FImplicitBVH::MakeEmpty()
		{
			return TUniquePtr<FImplicitBVH>(new FImplicitBVH());
		}

		TUniquePtr<FImplicitBVH> FImplicitBVH::TryMake(const TArrayView<const Chaos::FImplicitObjectPtr>& InRootObjects, const int32 InMinObjects, const int32 InMaxBVHDepth)
		{
			TArray<FImplicitBVHObject> LeafObjects = CollectLeafObjects(InRootObjects);
			return TryMakeFromLeaves(MoveTemp(LeafObjects), InMinObjects, InMaxBVHDepth);
		}
		
		TUniquePtr<FImplicitBVH> FImplicitBVH::TryMakeFromLeaves(TArray<FImplicitBVHObject>&& LeafObjects, const int32 InMinObjects, const int32 InMaxBVHDepth)
		{
			if (LeafObjects.Num() > InMinObjects)
			{
				return TUniquePtr<FImplicitBVH>(new FImplicitBVH(MoveTemp(LeafObjects), InMaxBVHDepth));
			}
			return TUniquePtr<FImplicitBVH>();
		}

		///////////////////////////////////////////////////////////////////////////////////////////
		///////////////////////////////////////////////////////////////////////////////////////////

		FImplicitBVH::FImplicitBVH()
			: NodeObjectIndices()
			, Nodes()
			, TreeDepth(0)
		{
		}

		FImplicitBVH::FImplicitBVH(FObjects&& InObjects, const int32 MaxDepth)
			: NodeObjectIndices()
			, Nodes()
			, TreeDepth(0)
		{
			Init(MoveTemp(InObjects), MaxDepth, 1);
		}

		FImplicitBVH::~FImplicitBVH()
		{
		}

		void FImplicitBVH::Init(FObjects&& InObjects, const int32 MaxDepth, const int32 MaxLeafObjects)
		{
			Objects = MoveTemp(InObjects);
			BuildTree(MaxDepth, MaxLeafObjects);
		}

		void FImplicitBVH::BuildTree(const int32 MaxDepth, const int32 MaxLeafObjects)
		{
			QUICK_SCOPE_CYCLE_COUNTER(Chaos_FImplicitBVH_BuildTree);

			Nodes.Reset();

			NodeObjectIndices.SetNumUninitialized(Objects.Num());

			FAABB3f NodeBounds = FAABB3f::EmptyAABB();

			FWorkData WorkData;
			WorkData.ObjectCenters.SetNumUninitialized(GetNumObjects());
			WorkData.NodeObjectIndices0.Reserve(Objects.Num());
			WorkData.NodeObjectIndices1.Reserve(Objects.Num());

			for (int32 ObjectIndex = 0; ObjectIndex < Objects.Num(); ++ObjectIndex)
			{
				const FImplicitBVHObject& Object = Objects[ObjectIndex];

				NodeObjectIndices[ObjectIndex] = ObjectIndex;

				NodeBounds.GrowToInclude(Object.GetBounds());

				WorkData.ObjectCenters[ObjectIndex] = Object.GetBoundsCenter();
			}

			AddNodeRecursive(NodeBounds, 0, GetNumObjects(), 0, MaxDepth, MaxLeafObjects, WorkData);
		}

		int32 FImplicitBVH::AddNodeRecursive(const FAABB3f& NodeBounds, const int32 ObjectBeginIndex, const int32 ObjectEndIndex, const int32 NodeDepth, const int32 MaxDepth, const int32 MaxLeafObjects, FWorkData& WorkData)
		{
			// Create a new node. By default this is a leaf node, but we may add children below
			const int32 NodeIndex = Nodes.Emplace(NodeBounds, ObjectBeginIndex, ObjectEndIndex);

			// Keep track of tree depth (used for stack sizing)
			TreeDepth = FMath::Max(NodeDepth + 1, TreeDepth);

			const int32 NumNodeObjects = ObjectEndIndex - ObjectBeginIndex;
			if ((NumNodeObjects > MaxLeafObjects) && (NodeDepth < MaxDepth - 1))
			{

				// NOTE: if we fail to partition, the node we just created will be a leaf node
				FAABB3f ChildNodeBounds0, ChildNodeBounds1;
				int32 ChildBeginIndex1;
				const bool bPartitioned = Partition(NodeBounds, ObjectBeginIndex, ObjectEndIndex, ChildNodeBounds0, ChildNodeBounds1, ChildBeginIndex1, WorkData);

				if (bPartitioned)
				{
					// @todo(chaos): could be run in parallel, but only worth it when each partition has a large number of children (and we couldn't share the WorkData arrays)
					Nodes[NodeIndex].ChildNodeIndices[0] = AddNodeRecursive(ChildNodeBounds0, ObjectBeginIndex, ChildBeginIndex1, NodeDepth + 1, MaxDepth, MaxLeafObjects, WorkData);
					Nodes[NodeIndex].ChildNodeIndices[1] = AddNodeRecursive(ChildNodeBounds1, ChildBeginIndex1, ObjectEndIndex, NodeDepth + 1, MaxDepth, MaxLeafObjects, WorkData);
				}
			}

			return NodeIndex;
		}

		bool FImplicitBVH::Partition(const FAABB3f& NodeBounds, const int32 ObjectBeginIndex, const int32 ObjectEndIndex, FAABB3f& OutChildNodeBounds0, FAABB3f& OutChildNodeBounds1, int32& OutPartitionIndex, FWorkData& WorkData)
		{
			OutChildNodeBounds0 = FAABB3f::EmptyAABB();
			OutChildNodeBounds1 = FAABB3f::EmptyAABB();
			OutPartitionIndex = INDEX_NONE;

			const int32 NumNodeObjects = ObjectEndIndex - ObjectBeginIndex;
			if (NumNodeObjects <= 0)
			{
				return false;
			}

			// Determine the split point with a nudge
			// @todo(chaos): currently using the mean center point - experiment with other options
			FVec3f MeanCenter = FVec3f(0);
			for (int32 NodeObjectIndex = ObjectBeginIndex; NodeObjectIndex < ObjectEndIndex; ++NodeObjectIndex)
			{
				MeanCenter += WorkData.ObjectCenters[NodeObjectIndices[NodeObjectIndex]];
			}
			MeanCenter /= FRealSingle(NumNodeObjects);
			const FVec3f SplitPoint = MeanCenter + FVec3(CVars::ChaosUnionBVHSplitBias);

			// Determine the split axis based on how many objects are exclusively one one side of the split
			// @todo(chaos): look at how much volume we add to each side of the split bounds instead?
			TVec3<int32> RejectLeft = TVec3<int32>(0);
			TVec3<int32> RejectRight = TVec3<int32>(0);
			for (int32 NodeObjectIndex = ObjectBeginIndex; NodeObjectIndex < ObjectEndIndex; ++NodeObjectIndex)
			{
				const FImplicitBVHObject& Object = Objects[NodeObjectIndices[NodeObjectIndex]];
				for (int32 Axis = 0; Axis < 3; ++Axis)
				{
					if (Object.GetBounds().Max()[Axis] < SplitPoint[Axis])
					{
						RejectLeft[Axis] += 1;
					}
					if (Object.GetBounds().Min()[Axis] > SplitPoint[Axis])
					{
						RejectRight[Axis] += 1;
					}
				}
			}

			// Pick the axis which has the most objects on both sides of the split
			int32 BestAxis = INDEX_NONE;
			int32 MaxRejects = 0;
			for (int32 Axis = 0; Axis < 3; ++Axis)
			{
				const int32 NumRejects = FMath::Min(RejectLeft[Axis], RejectRight[Axis]);
				if (NumRejects > MaxRejects)
				{
					MaxRejects = NumRejects;
					BestAxis = Axis;
				}
			}

			// If all objects span the split point on all axes, there's no point dividing further
			// @todo(chaos): we could try to choose a different split point
			if (BestAxis == INDEX_NONE)
			{
				return false;
			}

			// Collect all the object indices and net bounds for each side of the split
			WorkData.NodeObjectIndices0.Reset(NumNodeObjects);
			WorkData.NodeObjectIndices1.Reset(NumNodeObjects);
			for (int32 NodeObjectIndex = ObjectBeginIndex; NodeObjectIndex < ObjectEndIndex; ++NodeObjectIndex)
			{
				const int32 ObjectIndex = NodeObjectIndices[NodeObjectIndex];
				const FImplicitBVHObject& Object = Objects[ObjectIndex];

				if (WorkData.ObjectCenters[ObjectIndex][BestAxis] < SplitPoint[BestAxis])
				{
					WorkData.NodeObjectIndices0.Add(ObjectIndex);
					OutChildNodeBounds0.GrowToInclude(Object.GetBounds());
				}
				else
				{
					WorkData.NodeObjectIndices1.Add(ObjectIndex);
					OutChildNodeBounds1.GrowToInclude(Object.GetBounds());
				}
			}

			// Update the node indices to reflect the partitioning
			int32 NextNodeObjectIndex = ObjectBeginIndex;
			for (int32 NodeObjectIndex0 : WorkData.NodeObjectIndices0)
			{
				NodeObjectIndices[NextNodeObjectIndex++] = NodeObjectIndex0;
			}
			for (int32 NodeObjectIndex1 : WorkData.NodeObjectIndices1)
			{
				NodeObjectIndices[NextNodeObjectIndex++] = NodeObjectIndex1;
			}
			check(NextNodeObjectIndex == ObjectEndIndex);

			// The point in NodeObjectIndices where we split the objects in two
			OutPartitionIndex = ObjectBeginIndex + WorkData.NodeObjectIndices0.Num();

			return (OutPartitionIndex != INDEX_NONE);
		}

		///////////////////////////////////////////////////////////////////////////////////////////
		///////////////////////////////////////////////////////////////////////////////////////////
		// 
		// SERIALIZATION
		// 
		///////////////////////////////////////////////////////////////////////////////////////////
		///////////////////////////////////////////////////////////////////////////////////////////

		FChaosArchive& operator<<(FChaosArchive& Ar, FImplicitBVH& BVH)
		{ 
			return BVH.Serialize(Ar);
		}

		FChaosArchive& operator<<(FChaosArchive& Ar, FImplicitBVHObject& BVHObject)
		{ 
			return BVHObject.Serialize(Ar);
		}

		FChaosArchive& FImplicitBVH::Serialize(FChaosArchive& Ar)
		{
			Ar << Objects;

			// @todo(chaos): add new object version and serialize new BVH data
			// Load BVH intop temp and throw away, or save empty BVH
			TBoundingVolumeHierarchy<FObjects, TArray<int32>> BVH;
			Ar << BVH;

			return Ar;
		}

		FChaosArchive& FImplicitBVHObject::Serialize(FChaosArchive& Ar)
		{
			Ar << Geometry;
			Ar << X;
			Ar << R[0]; Ar << R[1]; Ar << R[2]; Ar << R[3];
			Ar << RootObjectIndex;
			return Ar;
		}

	} // namespace Private
} // namespace Chaos