// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/ImplicitObjectUnion.h"

#include "Chaos/ImplicitObjectBVH.h"
#include "Chaos/BoundingVolumeHierarchy.h"
#include "Chaos/Convex.h"
#include "Chaos/Framework/ArrayAlgorithm.h"
#include "Chaos/Tribox.h"

#include "UObject/FortniteMainBranchObjectVersion.h"

//UE_DISABLE_OPTIMIZATION

namespace Chaos
{
	namespace CVars
	{
		// Set to false to prevent BVHs from being created in collision detection (or used, if disabled in the runtime after creation)
		bool bChaosUnionBVHEnabled = true;
		FAutoConsoleVariableRef CVarChaosUnionBVHEnabled(TEXT("p.Chaos.Collision.UnionBVH.Enabled"), bChaosUnionBVHEnabled, TEXT("Set to false to disable use of BVH during collision detection (without affecting creations and serialization)"));

		// If a geometry hierarchy has more shapes than this, create a BVH around it
		int32 ChaosUnionBVHMinShapes = 10;
		FAutoConsoleVariableRef CVarChaosUnionBVHMinShapes(TEXT("p.Chaos.Collision.UnionBVH.NumShapes"), ChaosUnionBVHMinShapes, TEXT("If a geometry hierarchy has this many shapes, wrap it in a BVH for collision detection (negative to disable BVH)"));

		// Maximum BVH depth. In general we want the BVH to generate leafs with a single item in them
		int32 ChaosUnionBVHMaxDepth = 14;
		FAutoConsoleVariableRef CVarChaosUnionBVHMaxDepth(TEXT("p.Chaos.Collision.UnionBVH.MaxDepth"), ChaosUnionBVHMaxDepth, TEXT("The allowed depth of the BVH when used to wrap a shape hiererchy"));

		// A common case if objects arranged in a regular grid which is bad for the splitting algorithm.
		// This prevents objects exactly in the middle of the bounds of a cell from being assigned to 
		// a random child. See FImplicitBVH::Partition.
		FRealSingle ChaosUnionBVHSplitBias = 0.1f;
		FAutoConsoleVariableRef CVarChaosUnionBVHSplitBias(TEXT("p.Chaos.Collision.UnionBVH.SplitBias"), ChaosUnionBVHSplitBias, TEXT(""));

		// Use the new FImplicitObject::CountLeafObjectsInHierarchy to count leaf objects
		// @todo: remove this when tested
		bool bChaosImplicitBVHOptimizedCountLeafObjects = true;
		FAutoConsoleVariableRef CVarChaosImplicitBVHOptimizedCountLeafObjects(TEXT("p.Chaos.Collision.UnionBVH.UseOptimizedCountLeafObjects"), bChaosImplicitBVHOptimizedCountLeafObjects, TEXT(""));

	}

inline FAABB3 CalculateObjectsBounds(const TArrayView<FImplicitObjectPtr>& Objects)
{
	if (Objects.IsEmpty())
	{
		// No geometry, so a point particle at our local origin
		return FAABB3::ZeroAABB();
	}

	FAABB3 Bounds = Objects[0]->BoundingBox();
	for (int32 ObjectIndex = 1; ObjectIndex < Objects.Num(); ++ObjectIndex)
	{
		Bounds.GrowToInclude(Objects[ObjectIndex]->BoundingBox());
	}
	return Bounds;
}

FImplicitObjectUnion::FImplicitObjectUnion() 
	: FImplicitObject(EImplicitObject::HasBoundingBox, ImplicitObjectType::Union)
	, MLocalBoundingBox(FAABB3::ZeroAABB())
	, NumLeafObjects(0)
	, Flags()
{
}

FImplicitObjectUnion::FImplicitObjectUnion(TArray<TUniquePtr<FImplicitObject>>&& Objects)
	: FImplicitObject(EImplicitObject::HasBoundingBox, ImplicitObjectType::Union)
	, MObjects()
	, NumLeafObjects(0)
	, Flags()
{
	check(false);
}

FImplicitObjectUnion::FImplicitObjectUnion(TArray<Chaos::FImplicitObjectPtr>&& Objects)
	: FImplicitObject(EImplicitObject::HasBoundingBox, ImplicitObjectType::Union)
	, MObjects(MoveTemp(Objects))
	, NumLeafObjects(0)
	, Flags()
{
	ensure(MObjects.Num());

	MLocalBoundingBox = CalculateObjectsBounds(MakeArrayView(MObjects));
	SetNumLeafObjects(Private::FImplicitBVH::CountLeafObjects(MakeArrayView(GetObjects())));
}

FImplicitObjectUnion::FImplicitObjectUnion(FImplicitObjectUnion&& Other)
	: FImplicitObject(EImplicitObject::HasBoundingBox, ImplicitObjectType::Union)
	, MObjects(MoveTemp(Other.MObjects))
	, MLocalBoundingBox(MoveTemp(Other.MLocalBoundingBox))
	, BVH(MoveTemp(Other.BVH))
	, NumLeafObjects(Other.NumLeafObjects)
{
	Flags.Bits = Other.Flags.Bits;
}

FImplicitObjectUnion::~FImplicitObjectUnion() = default;

bool FImplicitObjectUnion::HasValidBVH() const
{
	// No need to create BVH if we are using the simple shape
	return BVH.IsValid() && CVars::bChaosUnionBVHEnabled;
}

void FImplicitObjectUnion::CheckNotLocked() const
{
	ensureMsgf(!IsLocked(), TEXT("Attempt to modify an ImplicitObjectUnion after it has been shared and locked"));
}
	
void FImplicitObjectUnion::Combine(TArray<Chaos::FImplicitObjectPtr>& OtherObjects)
{
	ensure(MObjects.Num());
	CheckNotLocked();

	const FAABB3 OtherBounds = CalculateObjectsBounds(MakeArrayView(OtherObjects));
	if (MObjects.Num() > 0)
	{
		MLocalBoundingBox.GrowToInclude(OtherBounds);
	}
	else
	{
		MLocalBoundingBox = OtherBounds;
	}
	MObjects.Reserve(MObjects.Num() + OtherObjects.Num());
	for (Chaos::FImplicitObjectPtr& ChildObject : OtherObjects)
	{
		SetNumLeafObjects(GetNumLeafObjects() + Private::FImplicitBVH::CountLeafObjects(MakeArrayView(&ChildObject, 1)));
		MObjects.Add(MoveTemp(ChildObject));
	}
	RebuildBVH();
}

void FImplicitObjectUnion::RemoveAt(int32 RemoveIndex)
{
	CheckNotLocked();

	if (RemoveIndex < MObjects.Num())
	{
		SetNumLeafObjects(GetNumLeafObjects() - Private::FImplicitBVH::CountLeafObjects(MakeArrayView(&MObjects[RemoveIndex], 1)));

		MObjects.RemoveAt(RemoveIndex);
	}

	MLocalBoundingBox = CalculateObjectsBounds(MakeArrayView(MObjects));
	RebuildBVH();
}

void FImplicitObjectUnion::RemoveAtSortedIndices(const TArrayView<const int32>& InIndices)
{
	CheckNotLocked();

	if (InIndices.IsEmpty())
	{
		return;
	}

	RemoveArrayItemsAtSortedIndices(MObjects, InIndices);

	SetNumLeafObjects(Private::FImplicitBVH::CountLeafObjects(MakeArrayView(GetObjects())));

	MLocalBoundingBox = CalculateObjectsBounds(MakeArrayView(MObjects));
	RebuildBVH();
}


void FImplicitObjectUnion::SetNumLeafObjects(int32 InNumLeafObjects)
{
	constexpr int32 MaxNumLeafObjects = int32(TNumericLimits<decltype(NumLeafObjects)>::Max());
	ensure(InNumLeafObjects <= MaxNumLeafObjects);
	check(InNumLeafObjects >= 0);

	NumLeafObjects = FMath::Min(InNumLeafObjects, MaxNumLeafObjects);
}

void FImplicitObjectUnion::CreateBVH()
{
	if (Flags.bAllowBVH && CVars::bChaosUnionBVHEnabled)
	{
		const int32 MinBVHShapes = CVars::ChaosUnionBVHMinShapes;
		const int32 MaxBVHDepth = CVars::ChaosUnionBVHMaxDepth;
		
		BVH = Private::FImplicitBVH::TryMake(MakeArrayView(MObjects), MinBVHShapes, MaxBVHDepth);
		Flags.bHasBVH = BVH.IsValid();
	}
}
	
void FImplicitObjectUnion::DestroyBVH()
{
	if (BVH.IsValid())
	{
		BVH.Reset();
		Flags.bHasBVH = false;
	}
}

void FImplicitObjectUnion::RebuildBVH()
{
	DestroyBVH();
	CreateBVH();
}

void FImplicitObjectUnion::FindAllIntersectingObjects(TArray<Pair<const FImplicitObject*,FRigidTransform3>>& Out, const FAABB3& LocalBounds) const
{
	if (HasValidBVH())
	{
		BVH->VisitOverlappingNodes(LocalBounds,
			[this, &Out](const int32 NodeIndex)
			{
				if (BVH->NodeIsLeaf(NodeIndex))
				{
					BVH->VisitNodeObjects(NodeIndex,
						[&Out](const FImplicitObject* Implicit, const FRigidTransform3f& RelativeTransformf,
							const FAABB3f& RelativeBoundsf, const int32 RootObjectIndex, const int32 LeafObjectIndex) -> void
						{
							Out.Add(MakePair(Implicit, FRigidTransform3(RelativeTransformf)));
						});
				}
			});
	}
	else
	{
		for (const Chaos::FImplicitObjectPtr& Object : MObjects)
        {
            Object->FindAllIntersectingObjects(Out, LocalBounds);
        }
	}
}

int32 FImplicitObjectUnion::CountObjectsInHierarchyImpl() const
{
	// Self
	int32 NumObjects = 1;
	
	// Children
	for (const Chaos::FImplicitObjectPtr& Object : GetObjects())
	{
		 NumObjects += Object->CountObjectsInHierarchy();
	}

	return NumObjects;
}

int32 FImplicitObjectUnion::CountLeafObjectsInHierarchyImpl() const
{
	// Do not count Self
	int32 NumObjects = 0;

	// Children
	for (const Chaos::FImplicitObjectPtr& Object : GetObjects())
	{
		NumObjects += Object->CountLeafObjectsInHierarchyImpl();
	}

	return NumObjects;
}

void FImplicitObjectUnion::VisitOverlappingLeafObjectsImpl(
	const FAABB3& LocalBounds,
	const FRigidTransform3& ObjectTransform,
	const int32 InRootObjectIndex,
	int32& ObjectIndex,
	int32& LeafObjectIndex,
	const FImplicitHierarchyVisitor& VisitorFunc) const
{
	// Skip self
	ObjectIndex++;

	if (HasValidBVH())
	{
		// Visit children
		// NOTE: ObjectIndex passed to the visitor isn't really correct here. Maybe it should be removed...
		BVH->VisitAllIntersections(LocalBounds,
			[this, &ObjectTransform, &ObjectIndex, &VisitorFunc](const FImplicitObject* Implicit, const FRigidTransform3f& RelativeTransformf, const FAABB3f& RelativeBoundsf, const int32 RootObjectIndex, const int32 LeafObjectIndex)
			{
				VisitorFunc(Implicit, FRigidTransform3(RelativeTransformf) * ObjectTransform, RootObjectIndex, ObjectIndex++, LeafObjectIndex);
			});
	}
	else
	{
		for (int32 BVHObjectIndex = 0; BVHObjectIndex < MObjects.Num(); ++BVHObjectIndex)
		{
			// If we are the root our object index is the root index, otherwise just pass on the value we were given (from the actual root)
			const int32 RootObjectIndex = (InRootObjectIndex != INDEX_NONE) ? InRootObjectIndex : BVHObjectIndex;
			MObjects[BVHObjectIndex]->VisitOverlappingLeafObjectsImpl(LocalBounds, ObjectTransform, RootObjectIndex, ObjectIndex, LeafObjectIndex, VisitorFunc);
		}
	}
}
	
void FImplicitObjectUnion::VisitLeafObjectsImpl(
	const FRigidTransform3& ObjectTransform,
	const int32 InRootObjectIndex,
	int32& ObjectIndex,
	int32& LeafObjectIndex,
	const FImplicitHierarchyVisitor& VisitorFunc) const
{
	// Skip self
	++ObjectIndex;
	
	for (int32 BVHObjectIndex = 0; BVHObjectIndex < MObjects.Num(); ++BVHObjectIndex)
	{
		// If we are the root our object index is the root index, otherwise just pass on the value we were given (from the actual root)
		const int32 RootObjectIndex = (InRootObjectIndex != INDEX_NONE) ? InRootObjectIndex : BVHObjectIndex;
		MObjects[BVHObjectIndex]->VisitLeafObjectsImpl(ObjectTransform, RootObjectIndex, ObjectIndex, LeafObjectIndex, VisitorFunc);
	}
}

bool FImplicitObjectUnion::VisitObjectsImpl(
	const FRigidTransform3& ObjectTransform,
	const int32 InRootObjectIndex,
	int32& ObjectIndex,
	int32& LeafObjectIndex,
	const FImplicitHierarchyVisitorBool& VisitorFunc) const
{
	// Visit self
	bool bContinue = VisitorFunc(this, ObjectTransform, InRootObjectIndex, ObjectIndex, INDEX_NONE);
	++ObjectIndex;
	
	// Visit Children
	for (int32 BVHObjectIndex = 0; (BVHObjectIndex < MObjects.Num()) && bContinue; ++BVHObjectIndex)
	{
		// If we are the root our object index is the root index, otherwise just pass on the value we were given (from the actual root)
		const int32 RootObjectIndex = (InRootObjectIndex != INDEX_NONE) ? InRootObjectIndex : BVHObjectIndex;
		bContinue = MObjects[BVHObjectIndex]->VisitObjectsImpl(ObjectTransform, RootObjectIndex, ObjectIndex, LeafObjectIndex, VisitorFunc);
	}

	return bContinue;
}

bool FImplicitObjectUnion::IsOverlappingBoundsImpl(const FAABB3& LocalBounds) const
{
	if (HasValidBVH())
	{
		return BVH->IsOverlappingBounds(LocalBounds);
	}
	else
	{
		if (LocalBounds.Intersects(BoundingBox()))
		{
			for (const Chaos::FImplicitObjectPtr& Object : MObjects)
			{
				if (Object->IsOverlappingBoundsImpl(LocalBounds))
				{
					return true;
				}
			}
		}
	}

	return false;
}

Chaos::FImplicitObjectPtr FImplicitObjectUnion::CopyGeometry() const
{
	TArray<Chaos::FImplicitObjectPtr> CopyOfObjects;
	CopyOfObjects.Reserve(MObjects.Num());
	for (const Chaos::FImplicitObjectPtr& Object : MObjects)
	{
		CopyOfObjects.Emplace(Object->CopyGeometry());
	}
	return MakeImplicitObjectPtr<FImplicitObjectUnion>(MoveTemp(CopyOfObjects));
}

Chaos::FImplicitObjectPtr FImplicitObjectUnion::CopyGeometryWithScale(const FVec3& Scale) const
{
	TArray<Chaos::FImplicitObjectPtr> CopyOfObjects;
	CopyOfObjects.Reserve(MObjects.Num());
	for (const Chaos::FImplicitObjectPtr& Object : MObjects)
	{
		CopyOfObjects.Emplace(Object->CopyGeometryWithScale(Scale));
	}
	return MakeImplicitObjectPtr<FImplicitObjectUnion>(MoveTemp(CopyOfObjects));
}

Chaos::FImplicitObjectPtr FImplicitObjectUnion::DeepCopyGeometry() const
{
	TArray<Chaos::FImplicitObjectPtr> CopyOfObjects;
	CopyOfObjects.Reserve(MObjects.Num());
	for (const Chaos::FImplicitObjectPtr& Object : MObjects)
	{
		CopyOfObjects.Emplace(Object->DeepCopyGeometry());
	}
	return MakeImplicitObjectPtr<FImplicitObjectUnion>(MoveTemp(CopyOfObjects));
}

Chaos::FImplicitObjectPtr FImplicitObjectUnion::DeepCopyGeometryWithScale(const FVec3& Scale) const
{
	TArray<Chaos::FImplicitObjectPtr> CopyOfObjects;
	CopyOfObjects.Reserve(MObjects.Num());
	for (const Chaos::FImplicitObjectPtr& Object : MObjects)
	{
		CopyOfObjects.Emplace(Object->DeepCopyGeometryWithScale(Scale));
	}
	return MakeImplicitObjectPtr<FImplicitObjectUnion>(MoveTemp(CopyOfObjects));
}

void FImplicitObjectUnion::ForEachObject(TFunctionRef<bool(const FImplicitObject&, const FRigidTransform3&)> Lambda) const
{
	// @todo(chaos): this implementation is strange. If we have as BVH we will visit all children in the hierarchy, but if not
	// we only visit our immediate children, and not their children. It should probably just ignore the BVH?
	if (HasValidBVH())
	{
		for (int32 Index = 0; Index < BVH->GetNumObjects(); ++Index)
		{
			if (const FImplicitObject* SubObject = BVH->GetGeometry(Index))
			{
				if (Lambda(*SubObject, BVH->GetTransform(Index)))
				{
					break;
				}
			}
		}
	}
	else
	{
		for (const Chaos::FImplicitObjectPtr& Object : MObjects)
		{
			if (Object)
			{
				if (Lambda(*Object, FRigidTransform3::Identity))
				{
					break;
				}
			}
		}
	}
}

void FImplicitObjectUnion::Serialize(FChaosArchive& Ar)
{
	Ar.UsingCustomVersion(FExternalPhysicsCustomObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteSeasonBranchObjectVersion::GUID);

	FChaosArchiveScopedMemory ScopedMemory(Ar, GetTypeName(), false);
	FImplicitObject::SerializeImp(Ar);
	Ar << MObjects;
	TBox<FReal, 3>::SerializeAsAABB(Ar, MLocalBoundingBox);

	bool bHierarchyBuilt = BVH.IsValid();
	if (Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) < FExternalPhysicsCustomObjectVersion::UnionObjectsCanAvoidHierarchy)
	{
		LegacySerializeBVH(Ar);
		Ar << bHierarchyBuilt;
	}
	else if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::ChaosImplicitObjectUnionBVHRefactor)
	{
		Ar << bHierarchyBuilt;
		if (bHierarchyBuilt)
		{
			LegacySerializeBVH(Ar);
		}
	}
	else
	{
		Ar << Flags.Bits;

		if (Ar.CustomVer(FFortniteSeasonBranchObjectVersion::GUID) < FFortniteSeasonBranchObjectVersion::ChaosImplicitObjectUnionLeafObjectsToInt32)
		{
			uint16 DummyNumLeafObjects = uint16(NumLeafObjects);
			Ar << DummyNumLeafObjects;
			NumLeafObjects = int32(DummyNumLeafObjects);
		}
		else
		{
			Ar << NumLeafObjects;
		}

		if (Flags.bHasBVH)
		{
			if (Ar.IsLoading())
			{
				BVH = Private::FImplicitBVH::MakeEmpty();
			}

			Ar << *BVH;

			if (Ar.IsLoading())
			{
				RebuildBVH();
			}
		}
	}

	if (Ar.IsLoading())
	{
		// We used to use empty bounds for particles with no geometry, but
		// now it is a point bounds at the local origin.
		if (MLocalBoundingBox.IsEmpty())
		{
			MLocalBoundingBox = FAABB3::ZeroAABB();
		}
	}
}

void FImplicitObjectUnion::LegacySerializeBVH(FChaosArchive& Ar)
{
	// We should only ever be loading old data. never saving it
	check(Ar.IsLoading());

	// The old data structure used FGeometryParticles which contains a lot of data we don't need
	struct FLargeImplicitObjectUnionData
	{
		FGeometryParticles GeomParticles;
		TBoundingVolumeHierarchy<FGeometryParticles, TArray<int32>> Hierarchy;
	};

	// Load the old data structure and chuck it away
	{
		FLargeImplicitObjectUnionData LegacyData;
		Ar << LegacyData.GeomParticles << LegacyData.Hierarchy;
	}

	// Count the objects in the hierarchy
	SetNumLeafObjects(Private::FImplicitBVH::CountLeafObjects(MakeArrayView(GetObjects())));

	// Only the root Union should allow BVH, but we don't know which that is at this stage
	// so just revert to the original behaviour of every Union potentially having a BVH
	Flags.bAllowBVH = true;
	RebuildBVH();
}

FImplicitObjectUnionClustered::FImplicitObjectUnionClustered()
	: FImplicitObjectUnion()
{
	Type = ImplicitObjectType::UnionClustered;
}

FImplicitObjectUnionClustered::FImplicitObjectUnionClustered(
	TArray<Chaos::FImplicitObjectPtr>&& Objects, 
	const TArray<FPBDRigidParticleHandle*>& OriginalParticleLookupHack)
    : FImplicitObjectUnion(MoveTemp(Objects))
	, MOriginalParticleLookupHack(OriginalParticleLookupHack)
{
	Type = ImplicitObjectType::UnionClustered;
	
	check(MOriginalParticleLookupHack.Num() == 0 || MOriginalParticleLookupHack.Num() == MObjects.Num());
	MCollisionParticleLookupHack.Reserve(FMath::Min(MOriginalParticleLookupHack.Num(), MObjects.Num()));
	for (int32 i = 0; MOriginalParticleLookupHack.Num() > 0 && i < MObjects.Num(); ++i)
	{
		// This whole part sucks, only needed because of how we get union 
		// children. Need to refactor and enforce no unions of unions.
		if (const TImplicitObjectTransformed<FReal, 3>* Transformed = 
			MObjects[i]->template GetObject<const TImplicitObjectTransformed<FReal, 3>>())
		{
			// Map const TImplicitObject<T,d>* to int32, where the latter
			// was the RigidBodyId
			MCollisionParticleLookupHack.Add(
				Transformed->GetTransformedObject(), MOriginalParticleLookupHack[i]);
		}
		else
		{
			ensure(false);	//shouldn't be here
		}
	}
}

FImplicitObjectUnionClustered::FImplicitObjectUnionClustered(FImplicitObjectUnionClustered&& Other)
	: FImplicitObjectUnion(MoveTemp(Other))
	, MOriginalParticleLookupHack(MoveTemp(MOriginalParticleLookupHack))
	, MCollisionParticleLookupHack(MoveTemp(MCollisionParticleLookupHack))
{
	Type = ImplicitObjectType::UnionClustered;
}

TArray<FPBDRigidParticleHandle*>
FImplicitObjectUnionClustered::FindAllIntersectingChildren(const FAABB3& LocalBounds) const
{
	TArray<FPBDRigidParticleHandle*> IntersectingChildren;
	if (HasValidBVH())
	{
		BVH->VisitOverlappingNodes(LocalBounds,
			[this, &IntersectingChildren](const int32 NodeIndex)
			{
				if (BVH->NodeIsLeaf(NodeIndex))
				{
					BVH->VisitNodeObjects(NodeIndex,
						[this, &IntersectingChildren](const FImplicitObject* Implicit, const FRigidTransform3f& RelativeTransformf, const FAABB3f& RelativeBoundsf, const int32 RootObjectIndex, const int32 LeafObjectIndex) -> void
						{
							if (ensure(MOriginalParticleLookupHack.IsValidIndex(LeafObjectIndex)))
							{
								IntersectingChildren.Add(MOriginalParticleLookupHack[LeafObjectIndex]);
							}
						});
				}
			});
	}
	else
	{
		// @todo(chaos): make this work when there's no BVH
		IntersectingChildren = MOriginalParticleLookupHack;
	}
	return IntersectingChildren;
}


const FPBDRigidParticleHandle* FImplicitObjectUnionClustered::FindParticleForImplicitObject(const FImplicitObject* Object) const
{
	typedef FPBDRigidParticleHandle* ValueType;

	const TImplicitObjectTransformed<FReal, 3>* AsTransformed = Object->template GetObject<TImplicitObjectTransformed<FReal, 3>>();
	if(AsTransformed)
	{
		const ValueType* Handle = MCollisionParticleLookupHack.Find(AsTransformed->GetTransformedObject());
		return Handle ? *Handle : nullptr;
	}

	const ValueType* Handle = MCollisionParticleLookupHack.Find(Object);
	return Handle ? *Handle : nullptr;
}

const FBVHParticles* FImplicitObjectUnionClustered::GetChildSimplicial(const int32 ChildIndex) const
{
	if (MOriginalParticleLookupHack.IsValidIndex(ChildIndex))
	{
		return MOriginalParticleLookupHack[ChildIndex]->CollisionParticles().Get();
	}
	return nullptr;
}

} // namespace Chaos
