// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/Interface/SQTypes.h"
#include "Chaos/ChaosArchive.h"
#include "Chaos/Serializable.h"
#include "Chaos/ParticleHandle.h"

namespace ChaosInterface
{
	void FActorShape::Serialize(Chaos::FChaosArchive& Ar)
	{
		Ar << Chaos::AsAlwaysSerializable(Actor);
		auto NonConstShape = const_cast<Chaos::FPerShapeData*>(Shape);
		Ar << Chaos::AsAlwaysSerializable(NonConstShape);
		Shape = NonConstShape;
	}

	void FQueryHit::Serialize(Chaos::FChaosArchive& Ar)
	{
		FActorShape::Serialize(Ar);
		Ar << FaceIndex;
	}

	void FLocationHit::Serialize(Chaos::FChaosArchive& Ar)
	{
		FQueryHit::Serialize(Ar);
		Ar << Flags;
		Ar << WorldPosition;
		Ar << WorldNormal;
		Ar << Distance;
	}

	void FRaycastHit::Serialize(Chaos::FChaosArchive& Ar)
	{
		FLocationHit::Serialize(Ar);
		Ar << U;
		Ar << V;
	}
}
