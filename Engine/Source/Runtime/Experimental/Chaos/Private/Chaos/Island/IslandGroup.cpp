// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/Island/IslandGroup.h"

#include "Chaos/ConstraintHandle.h"
#include "Chaos/Island/IslandManager.h"
#include "Chaos/PBDConstraintContainer.h"

namespace Chaos::CVars
{
	extern int32 ChaosSolverPositionIterations;
	extern int32 ChaosSolverVelocityIterations;
	extern int32 ChaosSolverProjectionIterations;
}

namespace Chaos
{
	namespace Private
	{
		FPBDIslandConstraintGroupSolver::FPBDIslandConstraintGroupSolver(FPBDIslandManager& InIslandManager)
			: FPBDConstraintGroupSolver()
			, IslandManager(InIslandManager)
			, NumParticles(0)
			, NumConstraints(0)
		{
		}

		void FPBDIslandConstraintGroupSolver::SetConstraintSolverImpl(int32 ContainerId)
		{
			if (ContainerId >= NumContainerConstraints.Num())
			{
				NumContainerConstraints.SetNumZeroed(ContainerId + 1);
			}
		}

		void FPBDIslandConstraintGroupSolver::ResetImpl()
		{
			NumParticles = 0;
			NumConstraints = 0;
			for (int32 ContainerIndex = 0; ContainerIndex < ConstraintContainerSolvers.Num(); ++ContainerIndex)
			{
				NumContainerConstraints[ContainerIndex] = 0;
			}

			Islands.Reset();
		}

		void FPBDIslandConstraintGroupSolver::AddIsland(FPBDIsland* Island)
		{
			if (Island)
			{
				Islands.Add(Island);

				NumParticles += Island->GetNumParticles();

				for (int32 ContainerIndex = 0; ContainerIndex < ConstraintContainerSolvers.Num(); ++ContainerIndex)
				{
					const int32 NumIslandConstraints = Island->GetNumContainerConstraints(ContainerIndex);
					NumContainerConstraints[ContainerIndex] += NumIslandConstraints;
					NumConstraints += NumIslandConstraints;
				}
			}
		}

		void FPBDIslandConstraintGroupSolver::AddConstraintsImpl()
		{
			// Initialize buffer sizes for solver particles
			SolverBodyContainer.Reset(NumParticles);

			// Add all the constraints to the solvers, but do not collect data from the constraints yet (each constraint type has its own solver). 
			// This also sets up the SolverBody array, but does not gather the body data.
			for (int32 ContainerIndex = 0; ContainerIndex < ConstraintContainerSolvers.Num(); ++ContainerIndex)
			{
				if (ConstraintContainerSolvers[ContainerIndex] != nullptr)
				{
					// Initialize buffer sizes for constraint solvers
					ConstraintContainerSolvers[ContainerIndex]->Reset(NumContainerConstraints[ContainerIndex]);

					for (FPBDIsland* Island : Islands)
					{
						TArrayView<FPBDIslandConstraint*> IslandConstraints = Island->GetConstraints(ContainerIndex);
						ConstraintContainerSolvers[ContainerIndex]->AddConstraints(IslandConstraints);
					}
				}
			}
		}

		void FPBDIslandConstraintGroupSolver::GatherBodiesImpl(const FReal Dt, const int32 BeginBodyIndex, const int32 EndBodyIndex)
		{
			// @todo(chaos): optimize
			for (int32 SolverBodyIndex = BeginBodyIndex; SolverBodyIndex < EndBodyIndex; ++SolverBodyIndex)
			{
				FSolverBody& SolverBody = SolverBodyContainer.GetSolverBody(SolverBodyIndex);
				FGeometryParticleHandle* Particle = SolverBodyContainer.GetParticle(SolverBodyIndex);
				const int32 Level = IslandManager.GetParticleLevel(Particle);
				SolverBody.SetLevel(Level);
			}
		}

		void FPBDIslandConstraintGroupSolver::SetIterationSettings(const FIterationSettings& InDefaultIterations)
		{
			// Start with undefined iteration counts
			Iterations = FIterationSettings::MakeEmpty();

			// Here we set the iteration count for the island group to be the largest of any of the islands.
			// @todo(chaos): we should support per-island iteration counts rather than pushing the max to all
			// island solved in the same group.
			for (FPBDIsland* Island : Islands)
			{
				Iterations = FIterationSettings::Merge(Iterations, Island->GetIterationSettings());
			}

			// If we still have undefined iterations counts, apply the defaults
			if (Iterations.GetNumPositionIterations() < 0)
			{
				Iterations.SetNumPositionIterations(InDefaultIterations.GetNumPositionIterations());
			}
			if (Iterations.GetNumVelocityIterations() < 0)
			{
				Iterations.SetNumVelocityIterations(InDefaultIterations.GetNumVelocityIterations());
			}
			if (Iterations.GetNumProjectionIterations() < 0)
			{
				Iterations.SetNumProjectionIterations(InDefaultIterations.GetNumProjectionIterations());
			}

			// Apply cvar overrides
			ApplyIterationSettingsOverrides();
		}

		void FPBDIslandConstraintGroupSolver::ApplyIterationSettingsOverrides()
		{
			if (CVars::ChaosSolverPositionIterations >= 0)
			{
				Iterations.SetNumPositionIterations(CVars::ChaosSolverPositionIterations);
			}
			if (CVars::ChaosSolverVelocityIterations >= 0)
			{
				Iterations.SetNumVelocityIterations(CVars::ChaosSolverVelocityIterations);
			}
			if (CVars::ChaosSolverProjectionIterations >= 0)
			{
				Iterations.SetNumProjectionIterations(CVars::ChaosSolverProjectionIterations);
			}
		}

	}	// namespace Private
}	// namespace Chaos