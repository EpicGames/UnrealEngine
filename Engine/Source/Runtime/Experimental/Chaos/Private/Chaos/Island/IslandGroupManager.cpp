// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/Island/IslandGroupManager.h"

#include "Chaos/ConstraintHandle.h"
#include "Chaos/Island/IslandManager.h"
#include "Chaos/PBDConstraintContainer.h"
#include "ChaosStats.h"
#include "ProfilingDebugging/ScopedTimers.h"

DECLARE_CYCLE_STAT(TEXT("Chaos::SolveConstraints"), STAT_SolveConstraints, STATGROUP_ChaosConstraintSolver);

DECLARE_CYCLE_STAT(TEXT("FPBDRigidsEvolutionGBF::Gather"), STAT_Evolution_Gather, STATGROUP_ChaosConstraintSolver);
DECLARE_CYCLE_STAT(TEXT("FPBDRigidsEvolutionGBF::Scatter"), STAT_Evolution_Scatter, STATGROUP_ChaosConstraintSolver);
DECLARE_CYCLE_STAT(TEXT("FPBDRigidsEvolutionGBF::ApplyConstraintsPhase1"), STAT_Evolution_ApplyConstraintsPhase1, STATGROUP_ChaosConstraintSolver);
DECLARE_CYCLE_STAT(TEXT("FPBDRigidsEvolutionGBF::ApplyConstraintsPhase2"), STAT_Evolution_ApplyConstraintsPhase2, STATGROUP_ChaosConstraintSolver);
DECLARE_CYCLE_STAT(TEXT("FPBDRigidsEvolutionGBF::ApplyConstraintsPhase3"), STAT_Evolution_ApplyConstraintsPhase3, STATGROUP_ChaosConstraintSolver);

DECLARE_CYCLE_STAT(TEXT("FPBDRigidsEvolutionGBF::Gather::Prepare"), STAT_Evolution_Gather_Prepare, STATGROUP_ChaosConstraintDetails);
DECLARE_CYCLE_STAT(TEXT("FPBDRigidsEvolutionGBF::Gather::Bodies"), STAT_Evolution_Gather_Bodies, STATGROUP_ChaosConstraintDetails);
DECLARE_CYCLE_STAT(TEXT("FPBDRigidsEvolutionGBF::Gather::Constraints"), STAT_Evolution_Gather_Constraints, STATGROUP_ChaosConstraintDetails);
DECLARE_CYCLE_STAT(TEXT("FPBDRigidsEvolutionGBF::Scatter::Bodies"), STAT_Evolution_Scatter_Bodies, STATGROUP_ChaosConstraintDetails);
DECLARE_CYCLE_STAT(TEXT("FPBDRigidsEvolutionGBF::Scatter::Constraints"), STAT_Evolution_Scatter_Constraints, STATGROUP_ChaosConstraintDetails);

namespace Chaos
{

	namespace CVars
	{
		// Different parallelism modes:
		//   0: Single-Threaded
		//   1: Parallel-For
		//   2: Tasks
		int32 GIslandGroupsParallelMode = 2;
		FAutoConsoleVariableRef GCVarIslandGroupsParallelMode(TEXT("p.Chaos.Solver.IslandGroups.ParallelMode"), GIslandGroupsParallelMode, TEXT("0: Single-Threaded; 1: Parallel-For; 2: Tasks"));

		// Cvar to control the number of island groups used in the solver. The total number will be NumThreads * GIslandGroupsWorkerMultiplier
		FRealSingle GIslandGroupsWorkerMultiplier = 1;
		FAutoConsoleVariableRef GCVarIslandGroupsWorkerThreadMultiplier(TEXT("p.Chaos.Solver.IslandGroups.WorkerMultiplier"), GIslandGroupsWorkerMultiplier, TEXT("Total number of island groups in the solver will be NumWorkerThreads * WorkerThreadMultiplier. [def:1]"));

		// Do not use more worker threads than this for the main solve (0 for unlimited)
		int32 GIslandGroupsMaxWorkers = 0;
		FAutoConsoleVariableRef GCVarIslandGroupsMaxWorkers(TEXT("p.Chaos.Solver.IslandGroups.MaxWorkers"), GIslandGroupsMaxWorkers, TEXT("The maximum number of worker threads to use (0 means unlimited)"));

		// We want a minimum number of constraints to gather/solve/scatter on each thread. This prevents us running too many tiny tasks on a many-core machine
		int32 GIslandGroupsMinConstraintsPerWorker = 50;
		FAutoConsoleVariableRef GCVarIslandGroupsMinConstraintsPerWorker(TEXT("p.Chaos.Solver.IslandGroups.MinConstraintsPerWorker"), GIslandGroupsMinConstraintsPerWorker, TEXT("The minimum number of constraints we want per worker thread"));

		// We want a minimum number of bodies to gather on each thread. This prevents us running too many tiny tasks on a many-core machine
		int32 GIslandGroupsMinBodiesPerWorker = 50;
		FAutoConsoleVariableRef GCVarIslandGroupsMinBodiesPerWorker(TEXT("p.Chaos.Solver.IslandGroups.MinBodiesPerWorker"), GIslandGroupsMinBodiesPerWorker, TEXT("The minimum number of bodies we want per worker thread"));
	}

	namespace Private
	{

		FPBDIslandGroupManager::FPBDIslandGroupManager(FPBDIslandManager& InIslandManager)
			: IslandManager(InIslandManager)
			, IslandGroups()
			, NumActiveGroups(0)
			, NumWorkerThreads(0)
			, TargetNumBodiesPerTask(0)
			, TargetNumConstraintsPerTask(0)
			, Iterations(0, 0, 0)
		{
			// Check for use of the "-onethread" command line arg, and physics threading disabled (GetNumWorkerThreads() is not affected by these)
			// @todo(chaos): is the number of worker threads a good indicator of how many threads we get in the solver loop? (Currently uses ParallelFor)
			NumWorkerThreads = (FApp::ShouldUseThreadingForPerformance() && !bSingleWorkerPhysics) ? FMath::Min(FTaskGraphInterface::Get().GetNumWorkerThreads(), Chaos::MaxNumWorkers) : 0;
			const int32 MaxIslandGroups = (CVars::GIslandGroupsMaxWorkers > 0) ? CVars::GIslandGroupsMaxWorkers : TNumericLimits<int32>::Max();
			const int32 NumIslandGroups = FMath::Clamp(FMath::CeilToInt32(FReal(NumWorkerThreads) * CVars::GIslandGroupsWorkerMultiplier), 1, MaxIslandGroups);

			IslandGroups.Reserve(NumIslandGroups);
			for (int32 GroupIndex = 0; GroupIndex < NumIslandGroups; ++GroupIndex)
			{
				IslandGroups.Emplace(MakeUnique<FPBDIslandConstraintGroupSolver>(IslandManager));
			}
		}

		FPBDIslandGroupManager::~FPBDIslandGroupManager()
		{

		}

		int32 FPBDIslandGroupManager::GetMaxSerialConstraintBatches(const int32 ContainerId) const
		{
			int32 MaxBatches = 0;
			for (int32 GroupIndex = 0; GroupIndex < NumActiveGroups; ++GroupIndex)
			{
				MaxBatches = FMath::Max(MaxBatches, IslandGroups[GroupIndex]->GetNumSerialConstraintBatches(ContainerId));
			}
			return MaxBatches;
		}

		void FPBDIslandGroupManager::AddConstraintContainer(FPBDConstraintContainer& ConstraintContainer, const int32 Priority)
		{
			const int32 ContainerId = ConstraintContainer.GetContainerId();

			for (int32 GroupIndex = 0; GroupIndex < IslandGroups.Num(); ++GroupIndex)
			{
				IslandGroups[GroupIndex]->SetConstraintSolver(ContainerId, ConstraintContainer.CreateGroupSolver(Priority));
			}
		}

		void FPBDIslandGroupManager::RemoveConstraintContainer(FPBDConstraintContainer& ConstraintContainer)
		{
			const int32 ContainerId = ConstraintContainer.GetContainerId();
			for (int32 GroupIndex = 0; GroupIndex < IslandGroups.Num(); ++GroupIndex)
			{
				IslandGroups[GroupIndex]->SetConstraintSolver(ContainerId, nullptr);
			}
		}

		void FPBDIslandGroupManager::SetConstraintContainerPriority(const int32 ContainerId, const int32 Priority)
		{
			for (int32 GroupIndex = 0; GroupIndex < IslandGroups.Num(); ++GroupIndex)
			{
				IslandGroups[GroupIndex]->SetConstraintSolverPriority(ContainerId, Priority);
			}
		}

		int32 FPBDIslandGroupManager::BuildGroups(const bool bIsResimming)
		{
			// The set of islands with some work to do
			// NOTE: We do not add islands with no constraints even if they have particles. There is no need since the Particles'
			// predicted positions/rotations will not be changed by the solver. (Zero-constraint islands may contain one isolated Particle).
			TArray<FPBDIsland*> Islands;
			for (int32 IslandIndex = 0; IslandIndex < IslandManager.GetNumIslands(); ++IslandIndex)
			{
				FPBDIsland* Island = IslandManager.GetIsland(IslandIndex); 

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
				if (Chaos::FPhysicsSolverBase::IsNetworkPhysicsPredictionEnabled() && Chaos::FPhysicsSolverBase::CanDebugNetworkPhysicsPrediction())
				{
					if(bIsResimming)
					{ 
						UE_LOGF(LogChaos, Log, "Chaos Island[%d] needs resim = %d", IslandIndex, Island->NeedsResim());
					}
				}
#endif
				if (!Island->IsSleeping() && (Island->GetNumConstraints() > 0) && (!bIsResimming || (bIsResimming && Island->NeedsResim())))
				{
					Islands.Add(Island);
				}
			}

			// Sort islands by constraint count - largest first
			// @todo(chaos): Do we need this?
			if (Islands.Num() > 1)
			{
				Algo::Sort(Islands, 
					[](const FPBDIsland* L, const FPBDIsland* R) -> bool
					{
						return L->GetNumConstraints() > R->GetNumConstraints();
					});
			}

			// Figure out how many bodies and constraints we'd like per group for gather, scatter and solve
			const int32 MaxGroups = IslandGroups.Num();
			int32 NumAllBodies = 0;
			int32 NumAllConstraints = 0;
			for (FPBDIsland* Island : Islands)
			{
				NumAllBodies += Island->GetNumParticles();
				NumAllConstraints += Island->GetNumConstraints();
			}
			TargetNumConstraintsPerTask = NumAllConstraints / MaxGroups + 1;
			TargetNumBodiesPerTask = NumAllBodies / MaxGroups + 1;

			// We want to gather or solve a minimum number of bodies and constraints per task if possible.
			// There's not point creating 100 tasks if each task only processes one constraint - the task overhead
			// would outweight the benefits of going wide.
			// @todo(chaos): we may want to consider separating the gather task count from the number of island groups by adding a second multiplier.
			TargetNumConstraintsPerTask = FMath::Max(TargetNumConstraintsPerTask, CVars::GIslandGroupsMinConstraintsPerWorker);
			TargetNumBodiesPerTask = FMath::Max(TargetNumConstraintsPerTask, CVars::GIslandGroupsMinBodiesPerWorker);

			// Reset all the groups
			for (TUniquePtr<FPBDIslandConstraintGroupSolver>& IslandGroup : IslandGroups)
			{
				IslandGroup->Reset();
			}
			NumActiveGroups = 0;

			// Add each Island to the first group with enough space, or the group with the fewest constraint if none have enough space
			// @todo(chaos): optimize - when a group is full we should move it to the back so we don't keep visiting it
			for (FPBDIsland* Island : Islands)
			{
				const int32 NumIslandConstraints = Island->GetNumConstraints();

				int32 InsertGroupIndex = INDEX_NONE;
				int32 SmallestGroupSize = TNumericLimits<int32>::Max();
				for (int32 GroupIndex = 0; GroupIndex < IslandGroups.Num(); ++GroupIndex)
				{
					const int32 NumGroupConstraints = IslandGroups[GroupIndex]->GetNumConstraints();

					if (NumGroupConstraints < SmallestGroupSize)
					{
						InsertGroupIndex = GroupIndex;
						SmallestGroupSize = NumGroupConstraints;
					}

					if (NumGroupConstraints + NumIslandConstraints < TargetNumConstraintsPerTask)
					{
						InsertGroupIndex = GroupIndex;
						break;
					}
				}

				check(InsertGroupIndex != INDEX_NONE);
				IslandGroups[InsertGroupIndex]->AddIsland(Island);

				NumActiveGroups = FMath::Max(NumActiveGroups, InsertGroupIndex + 1);
			}

			// Allow each island group to determine how many iterations to run
			for (TUniquePtr<FPBDIslandConstraintGroupSolver>& IslandGroup : IslandGroups)
			{
				IslandGroup->SetIterationSettings(Iterations);
			}

			return NumActiveGroups;
		}

		void FPBDIslandGroupManager::BuildGatherBatches(TArray<FIslandGroupRange>& BodyRanges, TArray<FIslandGroupRange>& ConstraintRanges)
		{
			BodyRanges.Reset();
			BodyRanges.Reserve(IslandGroups.Num());
			ConstraintRanges.Reset();
			ConstraintRanges.Reserve(IslandGroups.Num());

			// If a group has a lot of constraints in it (compared to the target number per task) split the gather into multiple tasks
			// The goal is to end up with the same number of tasks as we have task threads (see comments in SolveParallelTasks for more info)
			for (int32 GroupIndex = 0; GroupIndex < NumActiveGroups; ++GroupIndex)
			{
				FPBDIslandConstraintGroupSolver* IslandGroup = GetGroup(GroupIndex);

				const int32 NumConstraints = IslandGroup->GetNumSolverConstraints();
				const int32 NumConstraintGatherTasks = FMath::Max(1, FMath::DivideAndRoundNearest(NumConstraints, TargetNumConstraintsPerTask));
				const int32 NumConstraintsPerTask = FMath::DivideAndRoundUp(NumConstraints, NumConstraintGatherTasks);

				for (int32 BeginIndex = 0; BeginIndex < NumConstraints; BeginIndex += NumConstraintsPerTask)
				{
					const int32 EndIndex = FMath::Min(BeginIndex + NumConstraintsPerTask, NumConstraints);

					ConstraintRanges.Emplace(IslandGroup, BeginIndex, EndIndex);
				}

				const int32 NumBodies = IslandGroup->GetNumSolverBodies();
				const int32 NumBodyGatherTasks = FMath::Max(1, FMath::DivideAndRoundNearest(NumBodies, TargetNumBodiesPerTask));
				const int32 NumBodiesPerTask = FMath::DivideAndRoundUp(NumBodies, NumBodyGatherTasks);

				for (int32 BeginIndex = 0; BeginIndex < NumBodies; BeginIndex += NumBodiesPerTask)
				{
					const int32 EndIndex = FMath::Min(BeginIndex + NumBodiesPerTask, NumBodies);

					BodyRanges.Emplace(IslandGroup, BeginIndex, EndIndex);
				}
			}
		}

		void FPBDIslandGroupManager::SolveGroupConstraintsSerial(int32 GroupIndex, const FReal Dt)
		{
			FPBDIslandConstraintGroupSolver* IslandGroup = GetGroup(GroupIndex);
			QUICK_SCOPE_CYCLE_COUNTER(STAT_Evolution_ApplyConstraintsAllPhases);
			{
				SCOPE_CYCLE_COUNTER(STAT_Evolution_ApplyConstraintsPhase1);
				CSV_SCOPED_ISLANDGROUP_TIMING_STAT(PerIslandSolve_ApplyTotalSerialized, GroupIndex);

				IslandGroup->PreApplyPositionConstraints(Dt);
				IslandGroup->ApplyPositionConstraints(Dt);
			}
			{
				SCOPE_CYCLE_COUNTER(STAT_Evolution_ApplyConstraintsPhase2);
				CSV_SCOPED_ISLANDGROUP_TIMING_STAT(PerIslandSolve_ApplyPushOutTotalSerialized, GroupIndex);

				IslandGroup->PreApplyVelocityConstraints(Dt);
				IslandGroup->ApplyVelocityConstraints(Dt);
			}
			{
				SCOPE_CYCLE_COUNTER(STAT_Evolution_ApplyConstraintsPhase3);
				CSV_SCOPED_ISLANDGROUP_TIMING_STAT(PerIslandSolve_ApplyProjectionTotalSerialized, GroupIndex);

				IslandGroup->PreApplyProjectionConstraints(Dt);
				IslandGroup->ApplyProjectionConstraints(Dt);
			}
		}

		void FPBDIslandGroupManager::SolveGroupConstraintsParallelCalledFromTask(int32 GroupIndex, const FReal Dt)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_Evolution_ApplyConstraintsAllPhases);

			FPBDIslandConstraintGroupSolver* IslandGroup = GetGroup(GroupIndex);
			const TArray<UE::Tasks::FTask>& FinalTasks = IslandGroup->ApplyConstraintsParallelTasks(Dt);
			
			// AddNested() will prevent the calling task from terminating until the last constraint group solver task has finished.				
			for (const UE::Tasks::FTask& Prerequisite : FinalTasks)
			{
				UE::Tasks::AddNested(Prerequisite);
			}
		}

		void FPBDIslandGroupManager::SolveGroupConstraintsParallelNotCalledFromTask(int32 GroupIndex, const FReal Dt)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_Evolution_ApplyConstraintsAllPhases);

			FPBDIslandConstraintGroupSolver* IslandGroup = GetGroup(GroupIndex);
			const TArray<UE::Tasks::FTask>& FinalTasks = IslandGroup->ApplyConstraintsParallelTasks(Dt);

			// Global Wait() needed if not called from within another task.
			UE::Tasks::Wait(FinalTasks);
		}

		void FPBDIslandGroupManager::SolveGroupConstraintsCalledFromTask(int32 GroupIndex, const FReal Dt)
		{
			CSV_SCOPED_TIMING_STAT(Chaos, ApplyConstraints);

			FPBDIslandConstraintGroupSolver* IslandGroup = GetGroup(GroupIndex);

			if (IslandGroup->SolveInParallel())
			{
				SolveGroupConstraintsParallelCalledFromTask(GroupIndex, Dt);
			}
			else
			{
				SolveGroupConstraintsSerial(GroupIndex, Dt);
			}
		}

		void FPBDIslandGroupManager::SolveGroupConstraintsNotCalledFromTask(int32 GroupIndex, const FReal Dt)
		{
			CSV_SCOPED_TIMING_STAT(Chaos, ApplyConstraints);

			FPBDIslandConstraintGroupSolver* IslandGroup = GetGroup(GroupIndex);

			if (IslandGroup->SolveInParallel())
			{
				SolveGroupConstraintsParallelNotCalledFromTask(GroupIndex, Dt);
			}
			else
			{
				SolveGroupConstraintsSerial(GroupIndex, Dt);
			}
		}

		void FPBDIslandGroupManager::SolveSerial(const FReal Dt)
		{
			SCOPE_CYCLE_COUNTER(STAT_SolveConstraints);

			for (int32 GroupIndex = 0; GroupIndex < NumActiveGroups; ++GroupIndex)
			{
				FPBDIslandConstraintGroupSolver* IslandGroup = GetGroup(GroupIndex);

				{
					SCOPE_CYCLE_COUNTER(STAT_Evolution_Gather);
					CSV_SCOPED_ISLANDGROUP_TIMING_STAT(PerIslandSolve_GatherTotalSerialized, GroupIndex);

					IslandGroup->AddConstraintsAndBodies();
					IslandGroup->GatherBodies(Dt, 0, IslandGroup->GetNumSolverBodies());
					IslandGroup->GatherConstraints(Dt, 0, IslandGroup->GetNumSolverConstraints());
				}

				SolveGroupConstraintsNotCalledFromTask(GroupIndex, Dt);

				{
					SCOPE_CYCLE_COUNTER(STAT_Evolution_Scatter);
					CSV_SCOPED_ISLANDGROUP_TIMING_STAT(PerIslandSolve_ScatterTotalSerialized, GroupIndex);
					IslandGroup->ScatterConstraints(Dt);
					IslandGroup->ScatterBodies(Dt);
				}
			}
		}

		void FPBDIslandGroupManager::SolveParallelFor(const FReal Dt)
		{
			SCOPE_CYCLE_COUNTER(STAT_SolveConstraints);

			TArray<FIslandGroupRange> BodyRanges;
			TArray<FIslandGroupRange> ConstraintRanges;

			{
				CSV_SCOPED_TIMING_STAT(PhysicsVerbose, StepSolver_Gather);

				// Add all the bodies and constraints to the groups in the required order
				PhysicsParallelFor(NumActiveGroups,
					[this](const int32 GroupIndex)
					{
						SCOPE_CYCLE_COUNTER(STAT_Evolution_Gather_Prepare);
						CSV_SCOPED_ISLANDGROUP_TIMING_STAT(PerIslandSolve_GatherTotalSerialized, GroupIndex);

						FPBDIslandConstraintGroupSolver* IslandGroup = GetGroup(GroupIndex);
						IslandGroup->AddConstraintsAndBodies();
					});

				// Generate body and constraint ranges for parallel gather
				// NOTE: Must be after AddConstraintsAndBodies for all groups
				BuildGatherBatches(BodyRanges, ConstraintRanges);

				// Gather all body data using all worker threads
				PhysicsParallelFor(BodyRanges.Num(),
					[this, &BodyRanges, Dt](const int32 RangeIndex)
					{
						SCOPE_CYCLE_COUNTER(STAT_Evolution_Gather_Bodies);
						CSV_SCOPED_ISLANDGROUP_TIMING_STAT(PerIslandSolve_GatherTotalSerialized, RangeIndex);

						FIslandGroupRange& Range = BodyRanges[RangeIndex];
						Range.IslandGroup->GatherBodies(Dt, Range.BeginIndex, Range.EndIndex);
					});

				// Gather all constraint data
				PhysicsParallelFor(ConstraintRanges.Num(),
					[this, &ConstraintRanges, Dt](const int32 RangeIndex)
					{
						SCOPE_CYCLE_COUNTER(STAT_Evolution_Gather_Constraints);
						CSV_SCOPED_ISLANDGROUP_TIMING_STAT(PerIslandSolve_GatherTotalSerialized, RangeIndex);

						FIslandGroupRange& Range = ConstraintRanges[RangeIndex];
						Range.IslandGroup->GatherConstraints(Dt, Range.BeginIndex, Range.EndIndex);
					});
			}
		
			// Solve all the constraints, each group in parallel
			PhysicsParallelFor(NumActiveGroups,
				[this, Dt](const int32 GroupIndex)
				{
					FPBDIslandConstraintGroupSolver* IslandGroup = GetGroup(GroupIndex);

					SolveGroupConstraintsNotCalledFromTask(GroupIndex, Dt);
				});

			{
				SCOPE_CYCLE_COUNTER(STAT_Evolution_Scatter);

				// Scatter constraints using all worker threads
				PhysicsParallelFor(ConstraintRanges.Num(),
					[this, &ConstraintRanges, Dt](const int32 RangeIndex)
					{
						SCOPE_CYCLE_COUNTER(STAT_Evolution_Scatter_Constraints);
						CSV_SCOPED_ISLANDGROUP_TIMING_STAT(PerIslandSolve_ScatterTotalSerialized, RangeIndex);

						FIslandGroupRange& Range = ConstraintRanges[RangeIndex];
						Range.IslandGroup->ScatterConstraints(Dt, Range.BeginIndex, Range.EndIndex);
					});

				// Scatter bodies using all worker threads
				PhysicsParallelFor(BodyRanges.Num(),
					[this, &BodyRanges, Dt](const int32 RangeIndex)
					{
						SCOPE_CYCLE_COUNTER(STAT_Evolution_Scatter_Bodies);
						CSV_SCOPED_ISLANDGROUP_TIMING_STAT(PerIslandSolve_ScatterTotalSerialized, RangeIndex);

						FIslandGroupRange& Range = BodyRanges[RangeIndex];
						Range.IslandGroup->ScatterBodies(Dt, Range.BeginIndex, Range.EndIndex);
					});
			}
		}

		template<typename Lambda>
		TArray<UE::Tasks::FTask> FPBDIslandGroupManager::DispatchBatchedTasks(
			Lambda BatchedTask, 
			const int32 NumTasks, 
			const int32 NumItemsPerTask,
			const int32 NumItems,
			const FReal Dt,
			const TArray<UE::Tasks::FTask>& Prereqs
			)
		{
			TArray<UE::Tasks::FTask> BatchedTasks;
			BatchedTasks.Reserve(NumTasks);
			for (int32 BeginIndex = 0; BeginIndex < NumItems; BeginIndex += NumItemsPerTask)
			{
				const int32 EndIndex = FMath::Min(BeginIndex + NumItemsPerTask, NumItems);
				BatchedTasks.Add(UE::Tasks::Launch(UE_SOURCE_LOCATION,
					[BatchedTask, BeginIndex, EndIndex, Dt]()
					{
						BatchedTask(Dt, BeginIndex, EndIndex);
					},
					Prereqs, LowLevelTasks::ETaskPriority::High));
			}
			return MoveTemp(BatchedTasks);
		}

		void FPBDIslandGroupManager::SolveParallelTasks(const FReal Dt)
		{
			SCOPE_CYCLE_COUNTER(STAT_SolveConstraints);

			TArray<UE::Tasks::FTask> GroupSolveTasks;
			GroupSolveTasks.Reserve(NumActiveGroups);

			// Each IslandGroup is independent of the others so they can all be solved in parallel.
			// In addition, the Gather and Scatter work within each group is highly parallelizable.
			//
			// In order to utilize all workers when we have a small number of groups or some are much
			// larger than others, we allow each group to generate multiple gather/scatter tasks.
			// The number of tasks each group generates depends on how many constraints it has and
			// TargetNumConstraintsPerTask, which was calculated in BuildGroups such that we end
			// up with the right number of gather tasks below.
			//
			for (int32 GroupIndex = 0; GroupIndex < NumActiveGroups; ++GroupIndex)
			{
				// NOTE: This task spawns other child tasks. We must run AddConstraintsAndBodies before we know how many
				// Gather and Scatter tasks we want to create.
				UE::Tasks::FTask GroupSolveTask = UE::Tasks::Launch(UE_SOURCE_LOCATION,
					[this, GroupIndex, Dt]()
					{
						FPBDIslandConstraintGroupSolver* IslandGroup = GetGroup(GroupIndex);

						// Add bodies and constraints to the solvers but do no real work. Cannot be further parallelized.
						{
							SCOPE_CYCLE_COUNTER(STAT_Evolution_Gather_Prepare);
							IslandGroup->AddConstraintsAndBodies();
						}

						// How many gather tasks should we be creating for this group?
						// NOTE: TargetNumBodiesPerTask and TargetNumConstraintsPerTask were calculated in BuildGroups
						// so that we end up with a reasonable number of Gather/Scatter tasks for the system's worker threads.
						const int32 NumBodies = IslandGroup->GetNumSolverBodies();
						const int32 NumBodyGatherTasks = FMath::Max(1, FMath::DivideAndRoundNearest(NumBodies, TargetNumBodiesPerTask));
						const int32 NumBodiesPerTask = FMath::DivideAndRoundUp(NumBodies, NumBodyGatherTasks);
						const int32 NumConstraints = IslandGroup->GetNumSolverConstraints();
						const int32 NumConstraintGatherTasks = FMath::Max(1, FMath::DivideAndRoundNearest(NumConstraints, TargetNumConstraintsPerTask));
						const int32 NumConstraintsPerTask = FMath::DivideAndRoundUp(NumConstraints, NumConstraintGatherTasks);

						// Gather Bodies. Can run as wide as we like.
						TArray<UE::Tasks::FTask> GatherBodyTasks = DispatchBatchedTasks(
							[IslandGroup](const FReal Dt, const int32 BeginIndex, const int32 EndIndex) 
							{
								SCOPE_CYCLE_COUNTER(STAT_Evolution_Gather_Bodies);
								CSV_SCOPED_TIMING_STAT(PhysicsVerbose, StepSolver_Gather);
								IslandGroup->GatherBodies(Dt, BeginIndex, EndIndex);
							}, 
							NumBodyGatherTasks, 
							NumBodiesPerTask, 
							NumBodies, 
							Dt, 
							{ 
								UE::Tasks::FTask() 
							}
						);
						
						// Gather Constraints after body gather is complete. Can run as wide as we like.
						// Passing GatherBodyTasks directly as prerequisites replaces the FNullGraphTask fan-in.
						TArray<UE::Tasks::FTask> GatherConstraintTasks = DispatchBatchedTasks(
							[IslandGroup](const FReal Dt, const int32 BeginIndex, const int32 EndIndex)
							{
								SCOPE_CYCLE_COUNTER(STAT_Evolution_Gather_Constraints);
								CSV_SCOPED_TIMING_STAT(PhysicsVerbose, StepSolver_Gather);
								IslandGroup->GatherConstraints(Dt, BeginIndex, EndIndex);
							},
							NumConstraintGatherTasks,
							NumConstraintsPerTask,
							NumConstraints,
							Dt,
							GatherBodyTasks
						);

						// Solve Constraints after gathering is complete. 
						// Can be further parallelized with SolveTask if IslandGroup->SolveInParallel().
						UE::Tasks::FTask SolveTask = UE::Tasks::Launch(UE_SOURCE_LOCATION,
							[this, GroupIndex, Dt]()
							{
								SolveGroupConstraintsCalledFromTask(GroupIndex, Dt);
							},
							GatherConstraintTasks, LowLevelTasks::ETaskPriority::High);

						// Scatter Constraints after the solve completes.
						TArray<UE::Tasks::FTask> ScatterConstraintTasks = DispatchBatchedTasks(
							[IslandGroup](const FReal Dt, const int32 BeginIndex, const int32 EndIndex)
							{
								SCOPE_CYCLE_COUNTER(STAT_Evolution_Scatter_Constraints);
								CSV_SCOPED_TIMING_STAT(PhysicsVerbose, StepSolver_Scatter);
								IslandGroup->ScatterConstraints(Dt, BeginIndex, EndIndex);
							},
							NumConstraintGatherTasks,
							NumConstraintsPerTask,
							NumConstraints,
							Dt,
							{ 
								SolveTask 
							}
						);

						// Scatter Bodies.
						//
						// Note: Executed after ScatterConstraints to avoid race condition given that ScatterConstraints
						// reads FSolverBody P/Q and ScatterBodies writes to them.
						TArray<UE::Tasks::FTask> ScatterBodyTasks = DispatchBatchedTasks(
							[IslandGroup](const FReal Dt, const int32 BeginIndex, const int32 EndIndex)
							{
								SCOPE_CYCLE_COUNTER(STAT_Evolution_Scatter_Bodies);
								CSV_SCOPED_TIMING_STAT(PhysicsVerbose, StepSolver_Scatter);
								IslandGroup->ScatterBodies(Dt, BeginIndex, EndIndex);
							},
							NumBodyGatherTasks,
							NumBodiesPerTask,
							NumBodies,
							Dt,
							ScatterConstraintTasks
						);

						// Add nested task dependency so that GroupSolveTask can only finish when all internally called tasks have finished.
						if (ScatterBodyTasks.IsEmpty())
						{
							// NOTE: If no solver bodies present, ScatterBodyTasks will be empty, link to ScatterConstraintTasks instead.
							for (UE::Tasks::FTask& ScatterConstraintTask : ScatterConstraintTasks)
							{
								UE::Tasks::AddNested(ScatterConstraintTask);
							}
						}
						else
						{
							for (UE::Tasks::FTask& ScatterBodyTask : ScatterBodyTasks)
							{
								UE::Tasks::AddNested(ScatterBodyTask);
							}
						}
					},
					LowLevelTasks::ETaskPriority::High);

				GroupSolveTasks.Add(GroupSolveTask);
			}

			// Wait for all tasks to complete.
			UE::Tasks::Wait(GroupSolveTasks);
		}

		void FPBDIslandGroupManager::SetIterationSettings(const FIterationSettings& InIterations)
		{
			Iterations = InIterations;
		}

		void FPBDIslandGroupManager::Solve(const FReal Dt)
		{
	#if CSV_PROFILER_STATS
			GroupStats.Reset();
			GroupStats.AddDefaulted(IslandGroups.Num());
	#endif

			// @todo(chaos): Remove SolveParallelFor when SolveParallelTasks has been thoroughly tested
			const bool bSingleThreaded = bSingleWorkerPhysics || (IslandGroups.Num() == 1);
			const int32 ParallelMode = bSingleThreaded ? 0 : CVars::GIslandGroupsParallelMode;
			switch (ParallelMode)
			{
			case 0:
				SolveSerial(Dt);
				break;
			case 1:
				SolveParallelFor(Dt);
				break;
			case 2:
				SolveParallelTasks(Dt);
				break;
			}

	#if CSV_PROFILER_STATS
			FIslandGroupStats FlattenedStats = FIslandGroupStats::Flatten(GroupStats);
			FlattenedStats.ReportStats();
	#endif
		}
	}	// namespace Private
}	// namespace Chaos
