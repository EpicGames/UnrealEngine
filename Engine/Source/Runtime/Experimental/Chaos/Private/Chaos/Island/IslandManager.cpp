// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/Island/IslandManager.h"
#include "Chaos/ChaosDebugDraw.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/ParticleIterator.h"
#include "Chaos/PBDConstraintContainer.h"
#include "Chaos/PBDRigidsSOAs.h"
#include "Chaos/PerParticleGravity.h"
#include "Chaos/Evolution/IterationSettings.h"
#include "Chaos/PhysicsMaterialUtilities.h"
#include "Tasks/Task.h"

#include "ChaosStats.h"


// Extra check for debug mode
// NOTE: Should be disabled for checkin (except debug builds)
#ifndef CHAOS_CONSTRAINTGRAPH_CHECK_ENABLED
#define CHAOS_CONSTRAINTGRAPH_CHECK_ENABLED (UE_BUILD_DEBUG)
#endif

#if CHAOS_CONSTRAINTGRAPH_CHECK_ENABLED
#define CHAOS_CONSTRAINTGRAPH_CHECK(X) check(X)
#else
#define CHAOS_CONSTRAINTGRAPH_CHECK(X)
#endif

#define CHAOS_CONSTRAINTGRAPH_VALIDATE_TEST(X) \
	bIsValid = bIsValid && ensure(X); \
	if (!bIsValid) { return false; }


DECLARE_CYCLE_STAT(TEXT("IslandManager::UpdateParticles"), STAT_IslandManager_UpdateParticles, STATGROUP_ChaosIslands);
DECLARE_CYCLE_STAT(TEXT("IslandManager::Merge"), STAT_IslandManager_MergeIslands, STATGROUP_ChaosIslands);
DECLARE_CYCLE_STAT(TEXT("IslandManager::Split"), STAT_IslandManager_SplitIslands, STATGROUP_ChaosIslands);
DECLARE_CYCLE_STAT(TEXT("IslandManager::Levels"), STAT_IslandManager_AssignLevels, STATGROUP_ChaosIslands);
DECLARE_CYCLE_STAT(TEXT("IslandManager::Finalize"), STAT_IslandManager_Finalize, STATGROUP_ChaosIslands);
DECLARE_CYCLE_STAT(TEXT("IslandManager::ComputeIterationSettings"), STAT_IslandManager_ComputeIterationSettings, STATGROUP_ChaosIslands);
DECLARE_CYCLE_STAT(TEXT("IslandManager::Validate"), STAT_IslandManager_Validate, STATGROUP_ChaosIslands);

namespace Chaos::CVars
{
	extern int32 ChaosSolverCollisionPositionShockPropagationIterations;
	extern int32 ChaosSolverCollisionVelocityShockPropagationIterations;
	extern bool bChaosSolverPersistentGraph;
	extern FRealSingle SmoothedPositionLerpRate;

	bool bChaosConstraintGraphValidate = (CHAOS_CONSTRAINTGRAPH_CHECK_ENABLED != 0);
	FAutoConsoleVariableRef CVarChaosConstraintGraphValidate(TEXT("p.Chaos.ConstraintGraph.Validate"), bChaosConstraintGraphValidate, TEXT("Enable per-tick ConstraintGraph validation checks/assertions"));

	/** Cvar to enable/disable the island sleeping */
	bool bChaosSolverSleepEnabled = true;
	FAutoConsoleVariableRef CVarChaosSolverSleepEnabled(TEXT("p.Chaos.Solver.Sleep.Enabled"), bChaosSolverSleepEnabled, TEXT("Islands of non - moving particles will be deactivated."));

	/** Cvar to enable/disable partial island sleeping for all islands */
	bool bChaosSolverPartialIslandSleep = false;
	FAutoConsoleVariableRef CVarChaosSolverPartialIslandSleep(TEXT("p.Chaos.Solver.Sleep.PartialIslandSleep"), bChaosSolverPartialIslandSleep, TEXT("[EXPERIMENTAL] Single non-moving particles in islands will be deactivated even if they are connected to moving particles."));

	/** Cvar to enable/disable partial island waking for all islands */
	bool bChaosSolverPartialIslandWake = true;
	FAutoConsoleVariableRef CVarChaosSolverPartialIslandWake(TEXT("p.Chaos.Solver.Sleep.PartialIslandSleep.PartialIslandWake"), bChaosSolverPartialIslandWake, TEXT("[EXPERIMENTAL] Safeguard only: New constraints and particle state changes will trigger partial wake events instead of waking up the entire island.[def:true]"));

	/** Cvar to enable/disable level-based waking for all islands */
	bool bChaosSolverLevelBasedWake = true;
	FAutoConsoleVariableRef CVarChaosSolverLevelBasedWake(TEXT("p.Chaos.Solver.Sleep.PartialIslandSleep.LevelBasedWake"), bChaosSolverLevelBasedWake, TEXT("[EXPERIMENTAL] Safeguard only: Wake events will always propagate to higher levels, i.e. away from the closest kinematic.[def:true]"));

	/** Cvar to enable/disable forcing a partial island waking for all islands even if all particles are already awake.
	This is just a safeguard to force resetting the particle sleep counter when a new wake event occurs. It should generally be set to false. */
	bool bChaosSolverForcePartialIslandWake = false;
	FAutoConsoleVariableRef CVarChaosSolverForcePartialIslandWake(TEXT("p.Chaos.Solver.Sleep.PartialIslandSleep.ForcePartialIslandWake"), bChaosSolverForcePartialIslandWake, TEXT("[EXPERIMENTAL] Safeguard only: We force partial wake events even if all particles in the island are already awake to guarantee resetting the particle sleep counter.[def:false]"));

	/** Cvar to enable/disable waking up the entire island if the state of any particle has changed (partial island sleeping only) */
	bool bChaosSolverWakeOnParticleStateChanged = false;
	FAutoConsoleVariableRef CVarChaosSolverWakeOnParticleStateChanged(TEXT("p.Chaos.Solver.Sleep.PartialIslandSleep.WakeOnParticleStateChanged"), bChaosSolverWakeOnParticleStateChanged, TEXT("[EXPERIMENTAL] Safeguard only: State changes of a particle will trigger the island to wake up entirely.[def:false]"));

	/** Cvar to enable/disable triggering wake up for newly detected constraints with moving particles only (partial island sleeping only) */
	bool bChaosSolverWakeOnMovingOnly = true;
	FAutoConsoleVariableRef CVarChaosSolverWakeOnMovingOnly(TEXT("p.Chaos.Solver.Sleep.PartialIslandSleep.WakeOnMovingOnly"), bChaosSolverWakeOnMovingOnly, TEXT("[EXPERIMENTAL] Safeguard only: Only newly detected constraints connected to a moving particle will trigger wake-up event.[def:true]"));

	/** Cvar to adjust post-step waking triggered by velocity solver impulses.
	* The solver impulse is converted to a change in velocity by division by the particle mass.
	* Ideal setting = 20.0f, i.e. change in velocity > 20 will trigger wake of the sleeping particle. */
	FRealSingle ChaosSolverPostStepWakeThreshold = 20.0f;
	FAutoConsoleVariableRef CVarChaosSolverPostStepWakeThreshold(TEXT("p.Chaos.Solver.Sleep.PartialIslandSleep.PostStepWakeThreshold"), ChaosSolverPostStepWakeThreshold, TEXT("[EXPERIMENTAL] Post-solver waking of sleeping particles: Minimum squared change of velocity induced by the velocity-based collision solver (VBD, 0 to disable).[def:20]"));

	/** Cvar to enable/disable momentum propagation through the constraint graph (partial island sleeping only) */
	FRealSingle ChaosSolverMomentumPropagation = 0.3f;
	FAutoConsoleVariableRef CVarChaosSolverMomentumPropagation(TEXT("p.Chaos.Solver.Sleep.PartialIslandSleep.MomentumPropagation"), ChaosSolverMomentumPropagation, TEXT("[EXPERIMENTAL] Ratio of the impact momentum transferred from one sleeping particle to another through a constraint during partial wake event (0 to disable, 1 to propagate without dissipation).[def:0.3]"));

	/** Cvar to enable/disable partial sleep for collision constraints only, i.e. the sleep state of particles connect by other constraint types must match (partial island sleeping only)
	* 0: never prevent partial sleeping for non-collision constraints, i.e. all particles are allowed to sleep independent of their neighbors' sleep state and constraint types. 
	* 1: propagate partial wake events through non-collision constraints, AND post-solver re-waking of sleeping particles connected to an awake particle by non-collision constraint
	* -1: propagate partial wake events through non-collision constraints, NO post-solver re-waking of sleeping particles connected to an awake particle by non-collision constraint */
	int32 ChaosSolverPartialSleepCollisionConstraintsOnly = 1;
	FAutoConsoleVariableRef CVarChaosSolverPartialSleepCollisionConstraintsOnly(TEXT("p.Chaos.Solver.Sleep.PartialIslandSleep.CollisionConstraintsOnly"), ChaosSolverPartialSleepCollisionConstraintsOnly, TEXT("[EXPERIMENTAL] Safeguard only: Prevent partial island sleep between non-collision constraints (0: off, 1: re-wake sleeping particles connected to awake particles by non-collision constraints, -1: propagate wake events through non-collision constraints).[def:1]"));

	/** Cvar to enable partial waking pre-integration (UpdateExplicitSleep). */
	bool bChaosSolverPartialWakePreIntegration = true;
	FAutoConsoleVariableRef CVarChaosSolverPartialWakePreIntegration(TEXT("p.Chaos.Solver.Sleep.PartialIslandSleep.PartialWakePreIntegration"), bChaosSolverPartialWakePreIntegration, TEXT("[EXPERIMENTAL] Safeguard only: Partial wake events also happen pre-integration when the user explicilty sets a particle transform or applies external impulses or forces.[def:true]"));

	/** Cvar to set the minimum ratio of particles which needs to be motionless to activate partial island sleep. */
	FRealSingle ChaosSolverMinMotionlessRatio = 0.1f;
	FAutoConsoleVariableRef CVarChaosSolverMinMotionlessRatio(TEXT("p.Chaos.Solver.Sleep.PartialIslandSleep.MinimumMotionlessRatio"), ChaosSolverMinMotionlessRatio, TEXT("[EXPERIMENTAL] Ratio of the particles which need to be motionless in an island for partial island sleep to become active (0 to always activate partial island sleep).[def:0.1]"));

	/** Cvar to set the minimum number of ticks the ratio of motionless particles needs to be exceeded before switching between partial and full island sleep. */
	int32 ChaosSolverSleepModeSwitchThreshold = 10;
	FAutoConsoleVariableRef CVarChaosSolverSleepModeSwitchThreshold(TEXT("p.Chaos.Solver.Sleep.PartialIslandSleep.SleepModeSwitchCounterThreshold"), ChaosSolverSleepModeSwitchThreshold, TEXT("[EXPERIMENTAL] Safeguard only: Number of steps MinimumMotionlessRatio needs to be satisfied before switching the between full and partial island sleep.[def:10]"));

	/** Cvar to adjust the linear wake threshold for partial waking */
	FRealSingle LinearPartialWakeThresholdMultiplier = 10.0f;
	FAutoConsoleVariableRef CVarChaosSolverLinearWakeThresholdMultiplier(TEXT("p.Chaos.Solver.Sleep.PartialIslandSleep.LinearWakeThresholdMultiplier"), LinearPartialWakeThresholdMultiplier, TEXT("[EXPERIMENTAL] A multiplier applied to LinearSleepThreshold for particle waking.[def:10]"));

	/** Cvar to adjust the angular wake threshold for partial waking */
	FRealSingle AngularPartialWakeThresholdMultiplier = 10.0f;
	FAutoConsoleVariableRef CVarChaosSolverAngularWakeThresholdMultiplier(TEXT("p.Chaos.Solver.Sleep.PartialIslandSleep.AngularWakeThresholdMultiplier"), AngularPartialWakeThresholdMultiplier, TEXT("[EXPERIMENTAL] A multiplier applied to AngularSleepThreshold for particle waking.[def:10]"));

	/** Cvar to enable validate constraint sleep state at the end of the step (debugging only) */
	bool bChaosSolverValidateConstraintSleepState = false;
	FAutoConsoleVariableRef ChaosSolverValidateConstraintSleepState(TEXT("p.Chaos.Solver.Sleep.PartialIslandSleep.ValidateConstraintSleepState"), bChaosSolverValidateConstraintSleepState, TEXT("[EXPERIMENTAL] Debugging only: Enable constraint sleep state validation checks/assertions at the end of the tick.[def:false]"));

	/** Cvar to override the sleep counter threshold if necessary */
	int32 ChaosSolverCollisionDefaultSleepCounterThreshold = 20;
	FAutoConsoleVariableRef CVarChaosSolverCollisionDefaultSleepCounterThreshold(TEXT("p.Chaos.Solver.Sleep.Defaults.SleepCounterThreshold"), ChaosSolverCollisionDefaultSleepCounterThreshold, TEXT("Default counter threshold for sleeping.[def:20]"));

	/** Cvar to override the sleep linear threshold if necessary */
	FRealSingle ChaosSolverCollisionDefaultLinearSleepThreshold = 0.001f; // .001 unit mass cm
	FAutoConsoleVariableRef CVarChaosSolverCollisionDefaultLinearSleepThreshold(TEXT("p.Chaos.Solver.Sleep.Defaults.LinearSleepThreshold"), ChaosSolverCollisionDefaultLinearSleepThreshold, TEXT("Default linear threshold for sleeping.[def:0.001]"));

	/** Cvar to override the sleep angular threshold if necessary */
	FRealSingle ChaosSolverCollisionDefaultAngularSleepThreshold = 0.0087f;  //~1/2 unit mass degree
	FAutoConsoleVariableRef CVarChaosSolverCollisionDefaultAngularSleepThreshold(TEXT("p.Chaos.Solver.Sleep.Defaults.AngularSleepThreshold"), ChaosSolverCollisionDefaultAngularSleepThreshold, TEXT("Default angular threshold for sleeping.[def:0.0087]"));

	/** The size of object for which the angular sleep threshold is defined. Large objects reduce the threshold propertionally. 0 means do not apply size scale. */
	// E.g., if ChaosSolverCollisionAngularSleepThresholdSize=100, an objects with a bounds of 500 will have 1/5x the sleep threshold.
	// We are effectively converting the angular threshold into a linear threshold calculated at the object extents.
	// @todo(chaos): male this a project setting or something
	FRealSingle ChaosSolverCollisionAngularSleepThresholdSize = 0;
	FAutoConsoleVariableRef CVarChaosSolverCollisionAngularSleepThresholdSize(TEXT("p.Chaos.Solver.Sleep.AngularSleepThresholdSize"), ChaosSolverCollisionAngularSleepThresholdSize, TEXT("Scales the angular threshold based on size (0 to disable size based scaling)"));

	/* Cvar to increase the sleep counter threshold for floating particles */
	int32 IsolatedParticleSleepCounterThresholdMultiplier = 1;
	FAutoConsoleVariableRef CVarChaosSolverIsolatedParticleSleepCounterThresholdMultiplier(TEXT("p.Chaos.Solver.Sleep.IsolatedParticle.CounterMultiplier"), IsolatedParticleSleepCounterThresholdMultiplier, TEXT("A multiplier applied to SleepCounterThreshold for floating particles"));

	/* Cvar to adjust the sleep linear threshold for floating particles */
	FRealSingle IsolatedParticleSleepLinearThresholdMultiplier = 1.0f;
	FAutoConsoleVariableRef CVarChaosSolverIsolatedParticleSleepLinearThresholdMultiplier(TEXT("p.Chaos.Solver.Sleep.IsolatedParticle.LinearMultiplier"), IsolatedParticleSleepLinearThresholdMultiplier, TEXT("A multiplier applied to SleepLinearThreshold for floating particles"));

	/* Cvar to adjust the sleep angular threshold for floating particles */
	FRealSingle IsolatedParticleSleepAngularThresholdMultiplier = 1.0f;
	FAutoConsoleVariableRef CVarChaosSolverIsolatedParticleSleepAngularThresholdMultiplier(TEXT("p.Chaos.Solver.Sleep.IsolatedParticle.AngularMultiplier"), IsolatedParticleSleepAngularThresholdMultiplier, TEXT("A multiplier applied to SleepAngularThreshold for floating particles"));

	/** Cvar to enable/disable computing max iterations if island is dirty */
	bool bChaosSolverComputeIterationSettings = true;
	FAutoConsoleVariableRef CVarChaosSolverComputeIterationSettingsEnabled(TEXT("p.Chaos.Solver.ComputeIterationSettings.Enabled"), bChaosSolverComputeIterationSettings, TEXT("Recompute iteration settings every time an island is changed"));

	bool bChaosRandomizeConstraintOrder = false;
	FAutoConsoleVariableRef CVarChaosRandomizeConstraintOrder(TEXT("p.Chaos.Solver.RandomizeConstraintOrder"), bChaosRandomizeConstraintOrder, TEXT("Randomize constraint order for testing"));
}


namespace Chaos::Private
{
	//////////////////////////////////////////////////////////////////////////////////////////////////
	//////////////////////////////////////////////////////////////////////////////////////////////////
	// 
	// Utility Functions
	// 
	//////////////////////////////////////////////////////////////////////////////////////////////////
	//////////////////////////////////////////////////////////////////////////////////////////////////


	bool IsParticleDynamic(const FGeometryParticleHandle* Particle)
	{
		return (Particle != nullptr) && ((Particle->ObjectState() == EObjectStateType::Dynamic) || (Particle->ObjectState() == EObjectStateType::Sleeping));
	}

	bool IsParticleSleeping(const FGeometryParticleHandle* Particle)
	{
		return (Particle != nullptr) && (Particle->ObjectState() == EObjectStateType::Sleeping);
	}

	bool WasParticleAsleep(const FGeometryParticleHandle* Particle)
	{
		if (const FPBDRigidParticleHandle* Rigid = Particle->CastToRigidParticle())
		{
			return Rigid->WasSleeping();
		}
		return false;
	}

	bool IsParticleMoving(const FGeometryParticleHandle* Particle)
	{
		bool bIsStationary = true;
		if (Particle != nullptr)
		{
			if (Particle->ObjectState() == EObjectStateType::Kinematic)
			{
				// For kinematic particles check whether their current mode and target will cause the particle to move
				// For all other modes (Velocity, None and Reset) check that the velocity is non-zero
				const FKinematicGeometryParticleHandle* Kinematic = Particle->CastToKinematicParticle();
				const FKinematicTarget& KinematicTarget = Kinematic->KinematicTarget();
				if (KinematicTarget.GetMode() == EKinematicTargetMode::Position)
				{
					bIsStationary = (Kinematic->GetX() - KinematicTarget.GetPosition()).IsZero() && (Kinematic->GetRf() * KinematicTarget.GetRotation().Inverse()).IsIdentity();
				}
				else
				{
					bIsStationary = Kinematic->GetV().IsZero() && Kinematic->GetW().IsZero();
				}
			}
			else
			{
				bIsStationary = (Particle->ObjectState() == EObjectStateType::Static) || (Particle->ObjectState() == EObjectStateType::Sleeping);
			}
		}
		return !bIsStationary;
	}

	bool ParticlesIslandNeedsResim(const FGeometryParticleHandle* Particle)
	{
		return (Particle != nullptr) && (Particle->SyncState() != ESyncState::InSync || Particle->ResimType() == EResimType::FullResim);
	}

	bool GetIslandParticleSleepThresholds(
		const FGeometryParticleHandle* Particle,
		const TArrayCollectionArray<TSerializablePtr<FChaosPhysicsMaterial>>* PhysicsMaterials,
		const TArrayCollectionArray<TUniquePtr<FChaosPhysicsMaterial>>* PerParticlePhysicsMaterials,
		const THandleArray<FChaosPhysicsMaterial>* SimMaterials,
		FRealSingle& OutSleepLinearThreshold,
		FRealSingle& OutSleepAngularThreshold,
		int& OutSleepCounterThreshold,
		FRealSingle& OutWakeLinearThreshold,
		FRealSingle& OutWakeAngularThreshold)
	{
		OutSleepLinearThreshold = 0;
		OutSleepAngularThreshold = 0;
		OutSleepCounterThreshold = TNumericLimits<int32>::Max();

		const FPBDRigidParticleHandle* Rigid = Particle->CastToRigidParticle();
		if ((Rigid != nullptr) && (Rigid->SleepType() != ESleepType::NeverSleep))
		{
			const FRealSingle ParticleSleepThresholdMultiplier = Rigid->SleepThresholdMultiplier();

			const FChaosPhysicsMaterial* PhysicsMaterial = Private::GetFirstPhysicsMaterial(Rigid, PhysicsMaterials, PerParticlePhysicsMaterials, SimMaterials);
			if (PhysicsMaterial != nullptr)
			{
				OutSleepLinearThreshold = ParticleSleepThresholdMultiplier * FRealSingle(PhysicsMaterial->SleepingLinearThreshold);
				OutSleepAngularThreshold = ParticleSleepThresholdMultiplier * FRealSingle(PhysicsMaterial->SleepingAngularThreshold);
				OutSleepCounterThreshold = PhysicsMaterial->SleepCounterThreshold;
			}
			else
			{
				OutSleepLinearThreshold = ParticleSleepThresholdMultiplier * CVars::ChaosSolverCollisionDefaultLinearSleepThreshold;
				OutSleepAngularThreshold = ParticleSleepThresholdMultiplier * CVars::ChaosSolverCollisionDefaultAngularSleepThreshold;
				OutSleepCounterThreshold = CVars::ChaosSolverCollisionDefaultSleepCounterThreshold;
			}

			// Adjust angular threshold for size. It is equivalent to converting the angular threshold into a linear
			// movement threshold at the extreme points on the particle.
			const FRealSingle AngularSleepThresholdSize = CVars::ChaosSolverCollisionAngularSleepThresholdSize;
			if ((AngularSleepThresholdSize > 0) && Rigid->HasBounds())
			{
				const FRealSingle RigidSize = FRealSingle(Rigid->LocalBounds().Extents().GetMax());
				if (RigidSize > AngularSleepThresholdSize)
				{
					const FRealSingle ThresholdScale = AngularSleepThresholdSize / RigidSize;
					OutSleepAngularThreshold *= ThresholdScale;
				}
			}

			// Set the wake threshold used in partial island waking based on the particle's sleep threshold
			OutWakeLinearThreshold = CVars::LinearPartialWakeThresholdMultiplier * OutSleepLinearThreshold;
			OutWakeAngularThreshold = CVars::AngularPartialWakeThresholdMultiplier * OutSleepAngularThreshold;

			return true;
		}

		return false;
	}

	bool GetIsolatedParticleSleepThresholds(
		const FGeometryParticleHandle* Particle,
		const TArrayCollectionArray<TSerializablePtr<FChaosPhysicsMaterial>>* PhysicsMaterials,
		const TArrayCollectionArray<TUniquePtr<FChaosPhysicsMaterial>>* PerParticlePhysicsMaterials,
		const THandleArray<FChaosPhysicsMaterial>* SimMaterials,
		FRealSingle& OutSleepLinearThreshold,
		FRealSingle& OutSleepAngularThreshold,
		int& OutSleepCounterThreshold)
	{
		FRealSingle OutWakeAngularThreshold, OutWakeLinearThreshold;
		if (GetIslandParticleSleepThresholds(Particle, PhysicsMaterials, PerParticlePhysicsMaterials, SimMaterials, OutSleepLinearThreshold, OutSleepAngularThreshold, OutSleepCounterThreshold,
			OutWakeLinearThreshold, OutWakeAngularThreshold))
		{
			// Sleep thresholds are tuned to hide minor collision jitter which isn't a problem for floating particles
			// so we scale the thresholds to make sleeping harder. E.g., to avoid going to sleep at the apex of an
			// upwards ballistic trajectory, or when a floating oscillating body reverses rotation, etc.
			OutSleepCounterThreshold *= FMath::Max(1, CVars::IsolatedParticleSleepCounterThresholdMultiplier);
			OutSleepLinearThreshold *= FMath::Max(0.0f, CVars::IsolatedParticleSleepLinearThresholdMultiplier);
			OutSleepAngularThreshold *= FMath::Max(0.0f, CVars::IsolatedParticleSleepAngularThresholdMultiplier);

			return true;
		}

		return false;
	}

	bool GetParticleDisableThresholds(
		const FGeometryParticleHandle* Particle,
		const TArrayCollectionArray<TSerializablePtr<FChaosPhysicsMaterial>>* PhysicsMaterials,
		const TArrayCollectionArray<TUniquePtr<FChaosPhysicsMaterial>>* PerParticlePhysicsMaterials,
		const THandleArray<FChaosPhysicsMaterial>* SimMaterials,
		FRealSingle& OutDisableLinearThreshold,
		FRealSingle& OutDisableAngularThreshold)
	{
		OutDisableLinearThreshold = 0;
		OutDisableAngularThreshold = 0;

		const FPBDRigidParticleHandle* Rigid = Particle->CastToRigidParticle();
		if ((Rigid != nullptr) && (Rigid->SleepType() != ESleepType::NeverSleep))
		{
			const FChaosPhysicsMaterial* PhysicsMaterial = Private::GetFirstPhysicsMaterial(Rigid, PhysicsMaterials, PerParticlePhysicsMaterials, SimMaterials);
			if (PhysicsMaterial != nullptr)
			{
				OutDisableLinearThreshold = FRealSingle(PhysicsMaterial->DisabledLinearThreshold);
				OutDisableAngularThreshold = FRealSingle(PhysicsMaterial->DisabledAngularThreshold);
				return true;
			}
		}

		return false;
	}

	template<typename TRigidParticleHandle>
	void InitParticleSleepMetrics(TRigidParticleHandle& Rigid, FReal Dt)
	{
		if (Dt > UE_SMALL_NUMBER)
		{
			Rigid.SetVSmooth(Rigid.GetV());
			Rigid.SetWSmooth(Rigid.GetW());
		}
	}

	template<typename TRigidParticleHandle>
	void UpdateParticleSleepMetrics(TRigidParticleHandle& Rigid, FReal Dt)
	{
		if (Dt > UE_SMALL_NUMBER)
		{
			const FReal SmoothRate = FMath::Clamp(CVars::SmoothedPositionLerpRate, 0.0f, 1.0f);
			const FVec3 VImp = FVec3::CalculateVelocity(Rigid.GetX(), Rigid.GetP(), Dt);
			const FVec3 WImp = FRotation3::CalculateAngularVelocity(Rigid.GetR(), Rigid.GetQ(), Dt);
			Rigid.SetVSmooth(FMath::Lerp(Rigid.VSmooth(), VImp, SmoothRate));
			Rigid.SetWSmooth(FMath::Lerp(Rigid.WSmooth(), WImp, SmoothRate));
		}
	}

	// Validate that the number of elements in NodeQueue and MomentumQueue is compatible with the wake algorithm.
	// a) ChaosSolverMomentumPropagation <= 0.0f: No momentum information is stored.
	// b) ChaosSolverMomentumPropagation > 0.0f: Both queues must have the same number of elements.
	inline void ValidateWakeQueues(TArray<FPBDIslandParticle*>& NodeQueue, TArray<FPBDParticleMomentumSq>& MomentumQueue)
	{
		check((CVars::ChaosSolverMomentumPropagation > 0.0f && NodeQueue.Num() == MomentumQueue.Num())
			|| (CVars::ChaosSolverMomentumPropagation <= 0.0f && MomentumQueue.Num() == 0));
	}

	//////////////////////////////////////////////////////////////////////////////////////////////////
	//////////////////////////////////////////////////////////////////////////////////////////////////
	// 
	// FPBDIslandParticle
	// 
	//////////////////////////////////////////////////////////////////////////////////////////////////
	//////////////////////////////////////////////////////////////////////////////////////////////////


	FPBDIslandParticle::FPBDIslandParticle()
	{
	}

	FPBDIslandParticle::FPBDIslandParticle(FGeometryParticleHandle* InParticle)
	{
		Reuse(InParticle);
	}

	FPBDIslandParticle::~FPBDIslandParticle()
	{
		Trash();
	}

	int32 FPBDIslandParticle::GetIslandId() const
	{
		if (Island != nullptr)
		{
			return Island->GetIslandId();
		}
		return INDEX_NONE;
	}

	void FPBDIslandParticle::Reuse(FGeometryParticleHandle* InParticle)
	{
		Particle = InParticle;

		if (Particle != nullptr)
		{
			Particle->SetConstraintGraphNode(this);
		}
	}

	void FPBDIslandParticle::Trash()
	{
		check((Particle == nullptr) || (Particle->GetConstraintGraphNode() == this));
		check(ArrayIndex == INDEX_NONE);
		check(Edges.IsEmpty());
		check(Island == nullptr);
		check(IslandArrayIndex == INDEX_NONE);
		check(ArrayIndex == INDEX_NONE);

		if (Particle != nullptr)
		{
			Particle->SetConstraintGraphNode(nullptr);
			Particle = nullptr;
		}

		Level = 0;
		Flags.Reset();
		VisitEpoch = INDEX_NONE;
		ResimFrame = INDEX_NONE;
	}


	//////////////////////////////////////////////////////////////////////////////////////////////////
	//////////////////////////////////////////////////////////////////////////////////////////////////
	// 
	// FPBDIslandConstraint
	// 
	//////////////////////////////////////////////////////////////////////////////////////////////////
	//////////////////////////////////////////////////////////////////////////////////////////////////


	FPBDIslandConstraint::FPBDIslandConstraint()
	{
	}

	FPBDIslandConstraint::FPBDIslandConstraint(const int32 InContainerId, FConstraintHandle* InConstraint)
	{
		Reuse(InContainerId, InConstraint);
	}

	FPBDIslandConstraint::~FPBDIslandConstraint()
	{
		Trash();
	}

	int32 FPBDIslandConstraint::GetIslandId() const
	{
		if (Island != nullptr)
		{
			return Island->GetIslandId();
		}
		return INDEX_NONE;
	}

	void FPBDIslandConstraint::Reuse(const int32 InContainerId, FConstraintHandle* InConstraint)
	{
		ContainerIndex = InContainerId;
		Constraint = InConstraint;

		if (Constraint != nullptr)
		{
			Constraint->SetConstraintGraphEdge(this);

			// Initialize edge state to match constraint state
			Flags.bIsSleeping = Constraint->IsSleeping();
		}
	}

	void FPBDIslandConstraint::Trash()
	{
		check((Constraint == nullptr) || (Constraint->GetConstraintGraphEdge() == this));
		check(Nodes[0] == nullptr);
		check(Nodes[1] == nullptr);
		check(NodeArrayIndices[0] == INDEX_NONE);
		check(NodeArrayIndices[1] == INDEX_NONE);
		check(Island == nullptr);
		check(IslandArrayIndex == INDEX_NONE);
		check(ArrayIndex == INDEX_NONE);

		if (Constraint != nullptr)
		{
			Constraint->SetConstraintGraphEdge(nullptr);
			Constraint = nullptr;
		}

		VisitEpoch = INDEX_NONE;
		ContainerIndex = INDEX_NONE;
		Level = INDEX_NONE;
		Flags.Reset();
	}


	//////////////////////////////////////////////////////////////////////////////////////////////////
	//////////////////////////////////////////////////////////////////////////////////////////////////
	// 
	// FPBDIsland
	// 
	//////////////////////////////////////////////////////////////////////////////////////////////////
	//////////////////////////////////////////////////////////////////////////////////////////////////


	FPBDIsland::FPBDIsland()
	{
		Reuse();
	}

	FPBDIsland::~FPBDIsland()
	{
		Trash();
	}

	void FPBDIsland::Reuse()
	{
		Flags.bIsSleepAllowed = true;
		Flags.bIsPartialSleepAllowedInAllNodes = true;
		Flags.bHasSleepingNode = false;
		Flags.bEnoughMotionlessNodesForPartialSleep = true;
		Flags.bIsSleeping = true;
	}

	void FPBDIsland::Trash()
	{
		// Print more information about remaining nodes to track down UE-296297
		checkf(Nodes.IsEmpty(), 
			TEXT("Number of nodes: %d, Nodes[0]: %s, Number of edges: %d"), 
			Nodes.Num(), 
			*(Nodes[0]->GetParticle()->GetDebugName()), 
			Nodes[0]->Edges.Num());
		check(NumEdges == 0);
		check(ArrayIndex == INDEX_NONE);

		MergeSet = nullptr;
		MergeSetIslandIndex = INDEX_NONE;
		SleepCounter = 0;
		DisableCounter = 0;
		ResimFrame = INDEX_NONE;
		Flags.Reset();
	}

	void FPBDIsland::UpdateSyncState()
	{
		Flags.bNeedsResim = false;
		ResimFrame = INDEX_NONE;

		// If any of our particles want a resim, the whole island is resimmed
		for (TArray<FPBDIslandConstraint*>& Edges : ContainerEdges)
		{
			for (FPBDIslandConstraint* Edge : Edges)
			{
				for (int32 NodeIndex = 0; NodeIndex < 2; ++NodeIndex)
				{
					if (FPBDIslandParticle* EdgeNode = Edge->Nodes[NodeIndex])
					{
						Flags.bNeedsResim = ((Flags.bNeedsResim | EdgeNode->Flags.bNeedsResim) != 0);
						ResimFrame = (ResimFrame == INDEX_NONE) ? EdgeNode->ResimFrame : FMath::Min(ResimFrame, EdgeNode->ResimFrame);
					}
				}
			}
		}
	}

	bool FPBDIsland::IsPartialSleepEnabled() const
	{
		return CVars::bChaosSolverPartialIslandSleep && !!Flags.bEnoughMotionlessNodesForPartialSleep && !!Flags.bIsPartialSleepAllowedInAllNodes;
	}

	//////////////////////////////////////////////////////////////////////////////////////////////////
	//////////////////////////////////////////////////////////////////////////////////////////////////
	// 
	// FPBDIslandMergeSet
	// 
	//////////////////////////////////////////////////////////////////////////////////////////////////
	//////////////////////////////////////////////////////////////////////////////////////////////////


	FPBDIslandMergeSet::FPBDIslandMergeSet()
	{
		Reuse();
	}

	FPBDIslandMergeSet::~FPBDIslandMergeSet()
	{
		Trash();
	}

	void FPBDIslandMergeSet::Reuse()
	{
	}

	void FPBDIslandMergeSet::Trash()
	{
		check(ArrayIndex == INDEX_NONE);

		Islands.Reset();
		NumEdges = 0;
	}


	//////////////////////////////////////////////////////////////////////////////////////////////////
	//////////////////////////////////////////////////////////////////////////////////////////////////
	// 
	// FPBDIslandManager
	// 
	//////////////////////////////////////////////////////////////////////////////////////////////////
	//////////////////////////////////////////////////////////////////////////////////////////////////


	FPBDIslandManager::FPBDIslandManager(FPBDRigidsSOAs& InParticles)
		: Particles(InParticles)
		, PhysicsMaterials(nullptr)
		, PerParticlePhysicsMaterials(nullptr)
		, SimMaterials(nullptr)
		, Nodes(1000)
		, Edges(1000)
		, Islands(1000)
		, MergeSets(1000)
	{
	}

	FPBDIslandManager::~FPBDIslandManager()
	{
	}

	int32 FPBDIslandManager::GetNextVisitEpoch()
	{
		// Don't wrap to negative because INDEX_NONE is used as "uninitialized epoch"
		if (NextVisitEpoch == TNumericLimits<int32>::Max())
		{
			NextVisitEpoch = 0;
		}
		return NextVisitEpoch++;
	}

	bool FPBDIslandManager::ShouldAssignLevels(const FPBDIsland* Island) const
	{
		// Partial island sleeping with level-based wake always needs levels
		return bAssignLevels || (Island->IsPartialSleepEnabled() && CVars::bChaosSolverLevelBasedWake);
	}

	void FPBDIslandManager::SetIsDeterministic(const bool bInIsDeterministic)
	{
		bIsDeterministic = bInIsDeterministic;
		ApplyDeterminism();
	}

	void FPBDIslandManager::SetAssignLevels(const bool bInAssignLevels)
	{
		bAssignLevels = bInAssignLevels;
	}

	void FPBDIslandManager::AddConstraintContainer(const FPBDConstraintContainer& Container)
	{
		if (ConstraintContainers.Num() < Container.GetContainerId() + 1)
		{
			ConstraintContainers.SetNumZeroed(Container.GetContainerId() + 1, EAllowShrinking::No);
		}
		ConstraintContainers[Container.GetContainerId()] = &Container;
	}

	void FPBDIslandManager::RemoveConstraintContainer(const FPBDConstraintContainer& Container)
	{
		for (int32 EdgeIndex = Edges.Num() - 1; EdgeIndex >= 0; --EdgeIndex)
		{
			if (Edges[EdgeIndex]->ContainerIndex == Container.GetContainerId())
			{
				RemoveConstraint(Edges[EdgeIndex]->Constraint);
			}
		}

		ConstraintContainers[Container.GetContainerId()] = nullptr;
	}

	void FPBDIslandManager::SetMaterialContainers(const TArrayCollectionArray<TSerializablePtr<FChaosPhysicsMaterial>>* InPhysicsMaterials, const TArrayCollectionArray<TUniquePtr<FChaosPhysicsMaterial>>* InPerParticlePhysicsMaterials, const THandleArray<FChaosPhysicsMaterial>* InSimMaterials)
	{
		PhysicsMaterials = InPhysicsMaterials;
		PerParticlePhysicsMaterials = InPerParticlePhysicsMaterials;
		SimMaterials = InSimMaterials;
	}

	void FPBDIslandManager::SetGravityForces(const FPerParticleGravity* InGravity)
	{
		Gravity = InGravity;
	}

	void FPBDIslandManager::SetDisableCounterThreshold(const int32 InDisableCounterThreshold)
	{
		DisableCounterThreshold = InDisableCounterThreshold;
	}

	int32 FPBDIslandManager::GetNumConstraintContainers() const
	{
		return ConstraintContainers.Num();
	}

	void FPBDIslandManager::Reset()
	{
		for (int32 EdgeIndex = Edges.Num() - 1; EdgeIndex >= 0; --EdgeIndex)
		{
			RemoveConstraint(Edges[EdgeIndex]->Constraint);
		}
		check(Edges.IsEmpty());

		for (int32 NodeIndex = Nodes.Num() - 1; NodeIndex >= 0; --NodeIndex)
		{
			RemoveParticle(Nodes[NodeIndex]->Particle);
		}
		check(Nodes.IsEmpty());

		MergeSets.Reset();
		Islands.Reset();
		Edges.Reset();
		Nodes.Reset();
	}

	int32 FPBDIslandManager::ReserveParticles(const int32 InNumParticles)
	{
		const int32 NumNodes = Nodes.Num();
		Nodes.Reserve(InNumParticles);
		return FMath::Max(0, InNumParticles - NumNodes);
	}

	int32 FPBDIslandManager::GetParticleLevel(FGeometryParticleHandle* Particle) const
	{
		if (const FPBDIslandParticle* Node = GetGraphNode(Particle))
		{
			return Node->Level;
		}
		return INDEX_NONE;
	}

	void FPBDIslandManager::UpdateParticleMaterial(FGeometryParticleHandle* Particle)
	{
		if (FPBDIslandParticle* Node = GetGraphNode(Particle))
		{
			if (Node->Flags.bIsDynamic)
			{
				UpdateGraphNodeSleepSettings(Node);
			}
		}
	}

	const FPBDIsland* FPBDIslandManager::GetParticleIsland(const FGeometryParticleHandle* Particle) const
	{
		if (const FPBDIslandParticle* Node = GetGraphNode(Particle))
		{
			return Node->Island;
		}
		return nullptr;
	}

	const FPBDIsland* FPBDIslandManager::GetConstraintIsland(const FConstraintHandle* Constraint) const
	{
		if (const FPBDIslandConstraint* Edge = GetGraphEdge(Constraint))
		{
			return Edge->Island;
		}
		return nullptr;
	}

	int32 FPBDIslandManager::GetIslandIndex(const FPBDIsland* Island) const
	{
		if (Island != nullptr)
		{
			return Island->GetArrayIndex();
		}
		return INDEX_NONE;
	}

	int32 FPBDIslandManager::GetIslandArrayIndex(const FPBDIslandConstraint* Edge) const
	{
		if (Edge != nullptr)
		{
			return Edge->IslandArrayIndex;
		}
		return INDEX_NONE;
	}

	TArray<const FPBDIsland*> FPBDIslandManager::FindParticleIslands(const FGeometryParticleHandle* Particle) const
	{
		TArray<const FPBDIsland*> ParticleIslands;
		FindParticleIslands(Particle, ParticleIslands);
		return ParticleIslands;
	}

	void FPBDIslandManager::FindParticleIslands(const FGeometryParticleHandle* Particle, TArray<const FPBDIsland*>& OutParticleIslands) const
	{
		if (const FPBDIslandParticle* Node = GetGraphNode(Particle))
		{
			if (Node->Flags.bIsDynamic)
			{
				// Dynamic particles are in one island
				if (Node->Island != nullptr)
				{
					OutParticleIslands.Add(Node->Island);
				}
			}
			else
			{
				// Kinematic particles are in many islands and we need to check the edges
				for (FPBDIslandConstraint* Edge : Node->Edges)
				{
					if (Edge->Island != nullptr)
					{
						OutParticleIslands.AddUnique(Edge->Island);
					}
				}
			}
		}
	}

	TArray<const FGeometryParticleHandle*> FPBDIslandManager::FindParticlesInIslands(const TArray<const FPBDIsland*> InIslands) const
	{
		TArray<const FGeometryParticleHandle*> IslandParticles;
		FindParticlesInIslands(InIslands, IslandParticles);
		return IslandParticles;
	}

	void FPBDIslandManager::FindParticlesInIslands(const TArray<const FPBDIsland*>& InIslands, TArray<const FGeometryParticleHandle*>& OutIslandParticles) const
	{
		for (const FPBDIsland* Island : InIslands)
		{
			for (const TArray<FPBDIslandConstraint*>& ContainerEdges : Island->ContainerEdges)
			{
				for (const FPBDIslandConstraint* Edge : ContainerEdges)
				{
					if (Edge->Nodes[0] != nullptr)
					{
						OutIslandParticles.AddUnique(Edge->Nodes[0]->GetParticle());
					}
					if (Edge->Nodes[1] != nullptr)
					{
						OutIslandParticles.AddUnique(Edge->Nodes[1]->GetParticle());
					}
				}
			}
		}
	}

	TArray<const FConstraintHandle*> FPBDIslandManager::FindConstraintsInIslands(const TArray<const FPBDIsland*> InIslands, int32 ContainerId) const
	{
		TArray<const FConstraintHandle*> IslandConstraints;

		for (const FPBDIsland* Island : InIslands)
		{
			for (const FPBDIslandConstraint* Edge : Island->ContainerEdges[ContainerId])
			{
				IslandConstraints.Add(Edge->GetConstraint());
			}
		}

		return IslandConstraints;
	}

	void FPBDIslandManager::WakeParticleIslands(FGeometryParticleHandle* Particle)
	{
		// We want to wake for at least one frame (i.e., prevent re-sleeping this frame)
		bool bIsSleepAllowed = false;

		// When we explicitly wake we reset sleep counters etc
		if (FPBDIslandParticle* Node = GetGraphNode(Particle))
		{
			// Make sure the node sleep state matches the particle state
			Node->Flags.bIsSleeping = FConstGenericParticleHandle(Particle)->IsSleeping();

			// NOTE: We could check Flags.bIsDynamic here, but checking the Island pointer means we 
			// are automatically handling the case where a kinematic with constraints on it 
			// was made dynamic right before calling WakeParticleIsland
			if (Node->Island != nullptr)
			{
				// We permit sleeping for this island if partial island sleeping and pre-integration waking are enabled.
				// This will allows us to perform partial island waking if a dynamic particle has been woken by the user or had its transform set explicitly.
				if (Node->Island->IsPartialSleepEnabled() && CVars::bChaosSolverPartialWakePreIntegration)
				{
					bIsSleepAllowed = true;
				}

				// This is a dynamic particle and we know its island
				EnqueueIslandCheckSleep(Node->Island, bIsSleepAllowed);
				UpdateIslandPartialSleepAllowed(Node->Island, Particle);
			}
			else
			{
				// This is a kinematic particle, visit our edges to find the islands we are in
				for (const FPBDIslandConstraint* Edge : Node->Edges)
				{
					EnqueueIslandCheckSleep(Edge->Island, bIsSleepAllowed);
					UpdateIslandPartialSleepAllowed(Edge->Island, Particle);
				}
			}
		}

		// If we are an isolated particle we keep our own sleep counter
		// NOTE: We won't even be in the graph if we are dynamic but have no constraints
		if (FPBDRigidParticleHandle* Rigid = Particle->CastToRigidParticle())
		{
			Rigid->SetSleepCounter(0);
		}
	}

	void FPBDIslandManager::SleepParticle(FGeometryParticleHandle* Particle)
	{
		// When we explicitly sleep we must check the island for sleeping before we update constraints
		// so that constraint sleep state matched their particles' sleep state.
		const bool bIsSleepAllowed = true;
		if (FPBDIslandParticle* Node = GetGraphNode(Particle))
		{
			Node->Flags.bIsSleeping = true;

			if ((Node->Island != nullptr) && !Node->Island->Flags.bIsSleeping)
			{
				// Set the checksleep flag for processing in See UpdateParticlesCheckSleep
				EnqueueIslandCheckSleep(Node->Island, bIsSleepAllowed);
			}
		}
	}

	void FPBDIslandManager::FlagParticleUpdated(const FGeometryParticleHandle* Particle)
	{
		if (CVars::bChaosSolverPartialIslandSleep && CVars::bChaosSolverPartialWakePreIntegration)
		{
			if (FPBDIslandParticle* Node = GetGraphNode(Particle))
			{
				FPBDIsland* Island = Node->Island;
				if (Island && Island->IsPartialSleepEnabled())
				{
					Node->Flags.bWasUpdated = true;
				}
			}
		}
	}

	bool FPBDIslandManager::WasParticleTransformSetExplicitly(const FGeometryParticleHandle* Particle)
	{
		if (CVars::bChaosSolverPartialIslandSleep && CVars::bChaosSolverPartialWakePreIntegration)
		{
			if (FPBDIslandParticle* Node = GetGraphNode(Particle))
			{
				FPBDIsland* Island = Node->Island;
				if (Island && Island->IsPartialSleepEnabled())
				{
					return Node->Flags.bWasTransformSetExplicitly;
				}
			}
		}
		return false;
	}

	void FPBDIslandManager::FlagParticleTransformSetExplicitly(const FGeometryParticleHandle* Particle)
	{
		if (CVars::bChaosSolverPartialIslandSleep && CVars::bChaosSolverPartialWakePreIntegration)
		{
			if (FPBDIslandParticle* Node = GetGraphNode(Particle))
			{
				FPBDIsland* Island = Node->Island;
				if (Island && Island->IsPartialSleepEnabled())
				{
					Node->Flags.bWasTransformSetExplicitly = true;
				}
			}
		}
	}

	void FPBDIslandManager::AddParticle(FGeometryParticleHandle* Particle)
	{
		// Particles get auto-added via AddConstraint (see CreateGraphEdge, GetOrCreateGraphNode)
		// But must be explicitly removed when destroyed.
	}

	void FPBDIslandManager::RemoveParticle(FGeometryParticleHandle* Particle)
	{
		if (FPBDIslandParticle* Node = GetGraphNode(Particle))
		{
			// Islands are awakened when we disable a particle in it
			WakeNodeIslands(Node);

			// Remove all the constraints attached to the particle
			RemoveParticleConstraints(Particle);

			// Remove from our island
			RemoveNodeFromIsland(Node);

			// Destroy the node
			DestroyGraphNode(Node);

			// We should not be in the graph anymore
			check(GetGraphNode(Particle) == nullptr);
		}
	}

	void FPBDIslandManager::UpdateParticles()
	{
		SCOPE_CYCLE_COUNTER(STAT_IslandManager_UpdateParticles);

		// Reset per-island flag before updating graph nodes and repopulating it.
		ResetIslandPartialSleepAllowed();

		// Process state changed from registered particles.
		// To reduce cache-misses it would be nice to iterate over transient particle handles 
		// here but the view we need (Particles.GetActiveDynamicMovingKinematicParticlesView()) 
		// actually holds non-transient handles so we lose the benefits.
		// We would also have to visit nodes that are not represented in that view
		for (int32 NodeIndex = 0; NodeIndex < Nodes.Num(); ++NodeIndex)
		{
			if (NodeIndex < Nodes.Num() - 1)
			{
				// Prefetch the next handle
				FPlatformMisc::Prefetch(Nodes[NodeIndex + 1]->Particle);
			}

			FPBDIslandParticle* Node = Nodes[NodeIndex];
			UpdateGraphNode(Node);
		}

		// The above loop may leave orphaned nodes in the Nodes array when dynamics are made
		// kinematic and we have removed any kinematic-kinematic constraints. We don't 
		// handle node removal above because updating one node to kinematic can result 
		// in removal of many nodes which is awkward while iterating
		for (int32 NodeIndex = Nodes.Num() - 1; NodeIndex >= 0; --NodeIndex)
		{
			FPBDIslandParticle* Node = Nodes[NodeIndex];
			check(Node != nullptr);
			
			if (Node->Edges.IsEmpty() && (Node->Island == nullptr))
			{
				// We have an orphaned node - it must be removed from the graph here because
				// all other loops are over islands and won't visit this node
				DestroyGraphNode(Node);
			}
			else
			{
				// Every dynamic node needs to be in an island, non-dynamic nodes cannot be in an island.
				ensureMsgf((Node->Flags.bIsDynamic && (Node->Island != nullptr)) || (!Node->Flags.bIsDynamic && (Node->Island == nullptr)),
					TEXT("Node->Flags.bIsDynamic: %d, Node->Island: %d, Node->Edges.IsEmpty(): %d, DebugName: %s"), 
					Node->Flags.bIsDynamic, Node->Island != nullptr, Node->Edges.IsEmpty(), *(Node->GetParticle()->GetDebugName()));
			}
		}
	}

	void FPBDIslandManager::AddConstraint(const int32 ContainerId, FConstraintHandle* Constraint, const TVec2<FGeometryParticleHandle*>& ConstrainedParticles)
	{
		if (GetGraphEdge(Constraint) == nullptr)
		{
			// Create the edge representing the constraint. Also creates the nodes if necessary
			if (FPBDIslandConstraint* Edge = CreateGraphEdge(ContainerId, Constraint, ConstrainedParticles))
			{
				// NOTE: Used only by partial island sleeping. 
				// We restrict setting this flag to prevent overhead of resetting it.
				if (CVars::bChaosSolverPartialIslandSleep)
				{
					Edge->Flags.bIsNew = true;

					// When a new constraint is added, it should not be sleeping.
					check(!(!!Edge->Flags.bIsNew && !!Edge->Flags.bIsSleeping));
				}

				// Put the edge/nodes into an island and flag islands for merging if necessary
				// NOTE: We do not wake the island here. It will be awake already if one of the
				// particles is awake. It both particles are asleep we do not want to wake them
				// here - it would break support for bStartAwake==false for connected bodies.
				AssignEdgeIsland(Edge);
			}
		}
	}

	void FPBDIslandManager::RemoveConstraint(FConstraintHandle* Constraint)
	{
		if (FPBDIslandConstraint* Edge = GetGraphEdge(Constraint))
		{
			FPBDIslandParticle* Node0 = Edge->Nodes[0];
			FPBDIslandParticle* Node1 = Edge->Nodes[1];

			// Wake the island when we remove a constraint
			const bool bIsSleepAllowed = true;
			EnqueueIslandCheckSleep(Edge->Island, bIsSleepAllowed);

			// Remove edge from the island
			RemoveEdgeFromIsland(Edge);

			// Destroy the edge (also disconnects it from its nodes)
			DestroyGraphEdge(Edge);
		}
	}

	void FPBDIslandManager::RemoveParticleConstraints(FGeometryParticleHandle* Particle)
	{
		if (FPBDIslandParticle* Node = GetGraphNode(Particle))
		{
			// Reverse loop because we remove as we iterate
			for (int32 EdgeIndex = Node->Edges.Num() - 1; EdgeIndex >= 0; --EdgeIndex)
			{
				// Print more information about remaining nodes to track down UE-297690
				checkf(EdgeIndex < Node->Edges.Num(),
					TEXT("Array index out of bounds: %d into an array of size %d, Node->GetDebugName() %s."),
					EdgeIndex, Node->Edges.Num(), 
					*(Node->GetParticle()->GetDebugName()));

				FPBDIslandConstraint* Edge = Node->Edges[EdgeIndex];

				// NOTE: Destroys the edge and possibly this and other nodes (if this was their last constraint)
				RemoveConstraint(Edge->Constraint);
			}
		}
	}

	void FPBDIslandManager::RemoveParticleContainerConstraints(FGeometryParticleHandle* Particle, const int32 ContainerId)
	{
		if (FPBDIslandParticle* Node = GetGraphNode(Particle))
		{
			// Reverse loop because we remove as we iterate
			for (int32 EdgeIndex = Node->Edges.Num() - 1; EdgeIndex >= 0; --EdgeIndex)
			{
				FPBDIslandConstraint* Edge = Node->Edges[EdgeIndex];
				if (Edge->ContainerIndex == ContainerId)
				{
					// NOTE: Destroys the edge and possibly this and other nodes (if this was their last constraint)
					RemoveConstraint(Edge->Constraint);
				}
			}
		}
	}

	void FPBDIslandManager::RemoveContainerConstraints(const int32 ContainerId)
	{
		// Reverse loop because we remove as we iterate
		for (int32 EdgeIndex = Edges.Num() - 1; EdgeIndex >= 0; --EdgeIndex)
		{
			FPBDIslandConstraint* Edge = Edges[EdgeIndex];
			if (Edge->ContainerIndex == ContainerId)
			{
				RemoveConstraint(Edge->Constraint);
			}
		}
	}

	void FPBDIslandManager::RemoveAllConstraints()
	{
		// Reverse loop because we remove as we iterate
		for (int32 EdgeIndex = Edges.Num() - 1; EdgeIndex >= 0; --EdgeIndex)
		{
			FPBDIslandConstraint* Edge = Edges[EdgeIndex];
			RemoveConstraint(Edge->Constraint);
		}
	}

	void FPBDIslandManager::WakeConstraintIsland(FConstraintHandle* Constraint)
	{
		if (FPBDIslandConstraint* Edge = GetGraphEdge(Constraint))
		{
			const bool bIsSleepAllowed = false;
			EnqueueIslandCheckSleep(Edge->Island, bIsSleepAllowed);
		}
	}

	void FPBDIslandManager::UpdateIslands()
	{
		ProcessIslands();
	}

	void FPBDIslandManager::UpdateSleep(const FReal Dt)
	{
		ProcessSleep(FRealSingle(Dt));
	}

	void FPBDIslandManager::UpdateDisable(TFunctionRef<void(FPBDRigidParticleHandle*)> ParticleDisableFunctor)
	{
		ProcessDisable(ParticleDisableFunctor);
	}

	void FPBDIslandManager::EndTick()
	{
		ApplyDeterminism();
	}

	int32 FPBDIslandManager::GetParticleLevel(const FPBDIslandParticle* Node) const
	{
		if (Node != nullptr)
		{
			return Node->Level;
		}
		return INDEX_NONE;
	}

	int32 FPBDIslandManager::GetParticleColor(const FPBDIslandParticle* Node) const
	{
		return INDEX_NONE;
	}

	int32 FPBDIslandManager::GetConstraintLevel(const FPBDIslandConstraint* Edge) const
	{
		if (Edge != nullptr)
		{
			return Edge->Level;
		}
		return INDEX_NONE;
	}

	int32 FPBDIslandManager::GetConstraintColor(const FPBDIslandConstraint* Edge) const
	{
		return INDEX_NONE;
	}

	int32 FPBDIslandManager::GetParticleResimFrame(const FGeometryParticleHandle* Particle) const
	{
		if (const FPBDIslandParticle* Node = GetGraphNode(Particle))
		{
			return Node->ResimFrame;
		}
		return INDEX_NONE;
	}

	void FPBDIslandManager::SetParticleResimFrame(FGeometryParticleHandle* Particle, const int32 ResimFrame)
	{
		if (FPBDIslandParticle* Node = GetGraphNode(Particle))
		{
			Node->ResimFrame = ResimFrame;
		}
	}

	void FPBDIslandManager::ResetParticleResimFrame(const int32 ResetFrame)
	{
		for (FPBDIslandParticle* Node : Nodes)
		{
			Node->ResimFrame = ResetFrame;
		}
	}

	FPBDIslandParticle* FPBDIslandManager::GetGraphNode(const FGeometryParticleHandle* Particle) const
	{
		if (Particle != nullptr)
		{
			return Particle->GetConstraintGraphNode();
		}
		return nullptr;
	}

	FPBDIslandParticle* FPBDIslandManager::GetGraphNode(const FTransientGeometryParticleHandle& Particle) const
	{
		return Particle.GetConstraintGraphNode();
	}

	FPBDIslandParticle* FPBDIslandManager::CreateGraphNode(FGeometryParticleHandle* Particle)
	{
		if (Particle != nullptr)
		{
			check(GetGraphNode(Particle) == nullptr);

			UE_MT_SCOPED_WRITE_ACCESS(GlobalNodesAccessDetector);
			FPBDIslandParticle* Node = Nodes.Alloc(Particle);

			// Initial state
			Node->Flags.bIsDynamic = IsParticleDynamic(Particle);
			Node->Flags.bIsSleeping = IsParticleSleeping(Particle);
			Node->Flags.bIsMoving = IsParticleMoving(Particle);
			Node->Flags.bNeedsResim = ParticlesIslandNeedsResim(Particle);
			UpdateGraphNodeSleepSettings(Node);

			return Node;
		}
		return nullptr;
	}

	FPBDIslandParticle* FPBDIslandManager::GetOrCreateGraphNode(FGeometryParticleHandle* Particle)
	{
		FPBDIslandParticle* Node = GetGraphNode(Particle);
		if (Node == nullptr)
		{
			Node = CreateGraphNode(Particle);
		}
		return Node;
	}

	void FPBDIslandManager::DestroyGraphNode(FPBDIslandParticle* Node)
	{
		if (Node != nullptr)
		{
			// We should have already been removed from any island
			check(Node->Island == nullptr);
			check(Node->IslandArrayIndex == INDEX_NONE);

			UE_MT_SCOPED_WRITE_ACCESS(GlobalNodesAccessDetector);
			Nodes.Free(Node);
		}
	}

	FPBDIslandConstraint* FPBDIslandManager::GetGraphEdge(const FConstraintHandle* Constraint)
	{
		check(Constraint != nullptr);

		return Constraint->GetConstraintGraphEdge();
	}

	const FPBDIslandConstraint* FPBDIslandManager::GetGraphEdge(const FConstraintHandle* Constraint) const
	{
		return const_cast<FPBDIslandManager*>(this)->GetGraphEdge(Constraint);
	}

	FPBDIslandConstraint* FPBDIslandManager::CreateGraphEdge(const int32 ContainerId, FConstraintHandle* Constraint, const TVec2<FGeometryParticleHandle*>& ConstrainedParticles)
	{
		check(Constraint != nullptr);
		check(GetGraphEdge(Constraint) == nullptr);

		// Make sure our particles are represented in the graph
		if (IsParticleDynamic(ConstrainedParticles[0]) || IsParticleDynamic(ConstrainedParticles[1]))
		{
			FPBDIslandParticle* Node0 = GetOrCreateGraphNode(ConstrainedParticles[0]);
			FPBDIslandParticle* Node1 = GetOrCreateGraphNode(ConstrainedParticles[1]);

			UE_MT_SCOPED_WRITE_ACCESS(GlobalEdgesAccessDetector);

			// Create the edge and bind it to the nodes
			FPBDIslandConstraint* Edge = Edges.Alloc(ContainerId, Constraint);

			BindEdgeToNodes(Edge, Node0, Node1);

			return Edge;
		}

		return nullptr;
	}

	void FPBDIslandManager::DestroyGraphEdge(FPBDIslandConstraint* Edge)
	{
		if (Edge != nullptr)
		{
			// We should have already been removed from any island
			check(Edge->Island == nullptr);
			check(Edge->IslandArrayIndex == INDEX_NONE);

			// Disconnect from the nodes
			UnbindEdgeFromNodes(Edge);

			UE_MT_SCOPED_WRITE_ACCESS(GlobalEdgesAccessDetector);
			Edges.Free(Edge);
		}
	}

	void FPBDIslandManager::BindEdgeToNodes(FPBDIslandConstraint* Edge, FPBDIslandParticle* Node0, FPBDIslandParticle* Node1)
	{
		check(Edge != nullptr);
		check(Edge->Nodes[0] == nullptr);
		check(Edge->Nodes[1] == nullptr);
		check(Edge->NodeArrayIndices[0] == INDEX_NONE);
		check(Edge->NodeArrayIndices[1] == INDEX_NONE);
		check(Edge->Island == nullptr);

		UE_MT_SCOPED_WRITE_ACCESS(LocalEdgeAccessDetector);
		UE_MT_SCOPED_WRITE_ACCESS(LocalNodeAccessDetector);

		if (Node0 != nullptr)
		{
			Edge->Nodes[0] = Node0;
			Edge->NodeArrayIndices[0] = Node0->Edges.Add(Edge);
		}
		if (Node1 != nullptr)
		{
			Edge->Nodes[1] = Node1;
			Edge->NodeArrayIndices[1] = Node1->Edges.Add(Edge);
		}

		// To retain order between ticks for constraints at the same level we assign a permanent sort key to
		// each constraint. This also allows us to be deterministic as long as constraints are added in the same order.
		Edge->LevelSortKey = NextLevelSortKey++;
	}

	void FPBDIslandManager::UnbindEdgeFromNodes(FPBDIslandConstraint* Edge)
	{
		UE_MT_SCOPED_WRITE_ACCESS(LocalEdgeAccessDetector);
		UE_MT_SCOPED_WRITE_ACCESS(LocalNodeAccessDetector);

		for (int32 NodeIndex = 0; NodeIndex < 2; ++NodeIndex)
		{
			if (FPBDIslandParticle* Node = Edge->Nodes[NodeIndex])
			{
				const int32 ArrayIndex = Edge->NodeArrayIndices[NodeIndex];

				// These checks are supposed to help print out more information about the crashes in UE-215521, UE-296295, UE-314497
				// If the array index is out of bounds, we print out information about "Edge", i.e. the edge we intended to remove. 
				checkf(ArrayIndex < Node->Edges.Num(), 
					TEXT("Array index out of bounds: %d into an array of size %d, Node: %s, Edge->Type: %s, Edge->Node[0]: %s, Edge->Node[1]: %s."),
					ArrayIndex, Node->Edges.Num(),
					*(Node->GetParticle()->GetDebugName()), 
					*(Edge->GetConstraint()->GetType().ToString()), 
					Edge->Nodes[0] ? *(Edge->Nodes[0]->GetParticle()->GetDebugName()) : *(FString()),
					Edge->Nodes[1] ? *(Edge->Nodes[1]->GetParticle()->GetDebugName()) : *(FString()));
				// If the array index is within the bounds but does not correspond to the constraint we intended to remove, 
				// we print out information about "Edge", i.e. the edge we intended to remove, and the edge we currently reference "Node->Edges[ArrayIndex]".
				checkf(Node->Edges[ArrayIndex] == Edge, 
					TEXT("Selected edge does not correspond to ArrayIndex. Node: %s, Edge->Type: %s, Edge->Node[0]: %s, Edge->Node[1]: %s, Node->Edges[ArrayIndex]->Type: %s, Edges[ArrayIndex]->Node[0]: %s, Edges[ArrayIndex]->Node[1]: %s."), 
					*(Node->GetParticle()->GetDebugName()), 
					*(Edge->GetConstraint()->GetType().ToString()), 
					Edge->Nodes[0] ? *(Edge->Nodes[0]->GetParticle()->GetDebugName()) : *(FString()),
					Edge->Nodes[1] ? *(Edge->Nodes[1]->GetParticle()->GetDebugName()) : *(FString()),
					*(Node->Edges[ArrayIndex]->GetConstraint()->GetType().ToString()), 
					Node->Edges[ArrayIndex]->Nodes[0] ? *(Node->Edges[ArrayIndex]->Nodes[0]->GetParticle()->GetDebugName()) : *(FString()),
					Node->Edges[ArrayIndex]->Nodes[1] ? *(Node->Edges[ArrayIndex]->Nodes[1]->GetParticle()->GetDebugName()) : *(FString()));
				

				// Remove the edge from the node
				Node->Edges.RemoveAtSwap(ArrayIndex, EAllowShrinking::No);
				Edge->Nodes[NodeIndex] = nullptr;
				Edge->NodeArrayIndices[NodeIndex] = INDEX_NONE;

				// Update the array index of the edge we swapped in
				if (ArrayIndex < Node->Edges.Num())
				{
					// We don't know which of the edges nodes we are...
					if (Node->Edges[ArrayIndex]->Nodes[0] == Node)
					{
						checkf(Node->Edges[ArrayIndex]->NodeArrayIndices[0] == Node->Edges.Num(), 
							TEXT("The node we swapped in was not at the end of the array. Node->Edges[ArrayIndex]->NodeArrayIndices[0]: %d, Node->Edges.Num(): %d, Node->DebugName: %s, Node->Edges[ArrayIndex]->Type: %s, Edges[ArrayIndex]->Node[0]: %s, Edges[ArrayIndex]->Node[1]: %s."),
							Node->Edges[ArrayIndex]->NodeArrayIndices[0], Node->Edges.Num(), 
							*(Node->GetParticle()->GetDebugName()),
							*(Node->Edges[ArrayIndex]->GetConstraint()->GetType().ToString()),
							Node->Edges[ArrayIndex]->Nodes[0] ? *(Node->Edges[ArrayIndex]->Nodes[0]->GetParticle()->GetDebugName()) : *(FString()),
							Node->Edges[ArrayIndex]->Nodes[1] ? *(Node->Edges[ArrayIndex]->Nodes[1]->GetParticle()->GetDebugName()) : *(FString()));

						Node->Edges[ArrayIndex]->NodeArrayIndices[0] = ArrayIndex;
					}
					else if (Node->Edges[ArrayIndex]->Nodes[1] == Node)
					{
						checkf(Node->Edges[ArrayIndex]->NodeArrayIndices[1] == Node->Edges.Num(),
							TEXT("The node we swapped in was not at the end of the array. Node->Edges[ArrayIndex]->NodeArrayIndices[1]: %d, Node->Edges.Num(): %d, Node: %s, Node->Edges[ArrayIndex]->Type: %s, Edges[ArrayIndex]->Node[0]: %s, Edges[ArrayIndex]->Node[1]: %s."),
							Node->Edges[ArrayIndex]->NodeArrayIndices[1], Node->Edges.Num(), 
							*(Node->GetParticle()->GetDebugName()),
							*(Node->Edges[ArrayIndex]->GetConstraint()->GetType().ToString()),
							Node->Edges[ArrayIndex]->Nodes[0] ? *(Node->Edges[ArrayIndex]->Nodes[0]->GetParticle()->GetDebugName()) : *(FString()),
							Node->Edges[ArrayIndex]->Nodes[1] ? *(Node->Edges[ArrayIndex]->Nodes[1]->GetParticle()->GetDebugName()) : *(FString()));

						Node->Edges[ArrayIndex]->NodeArrayIndices[1] = ArrayIndex;
					}
					else
					{
						// If we get here, the list Node->Edges must be corrupted.
						checkf(false, 
							TEXT("Node %s was not found between Node->Edges[ArrayIndex]->Nodes[0] %s and Node->Edges[ArrayIndex]->Nodes[1] %s."),
							*(Node->GetParticle()->GetDebugName()), 
							*(Node->Edges[ArrayIndex]->Nodes[0]->GetParticle()->GetDebugName()),
							*(Node->Edges[ArrayIndex]->Nodes[1]->GetParticle()->GetDebugName()));
					}
				}
			}
		}
	}

	void FPBDIslandManager::UpdateGraphNodeSleepSettings(FPBDIslandParticle* Node)
	{
		check(Node != nullptr);

		if (Node->Flags.bIsDynamic)
		{
			FRealSingle SleepLinearThreshold, SleepAngularThreshold;
			FRealSingle WakeLinearThreshold, WakeAngularThreshold;
			FRealSingle DisableLinearThreshold, DisableAngularThreshold;
			int32 SleepCounterThreshold;
			GetIslandParticleSleepThresholds(Node->Particle, PhysicsMaterials, PerParticlePhysicsMaterials, SimMaterials, SleepLinearThreshold, SleepAngularThreshold,
				SleepCounterThreshold, WakeLinearThreshold, WakeAngularThreshold);
			GetParticleDisableThresholds(Node->Particle, PhysicsMaterials, PerParticlePhysicsMaterials, SimMaterials, DisableLinearThreshold, DisableAngularThreshold);

			Node->SleepLinearThresholdSq = FMath::Square(SleepLinearThreshold);
			Node->SleepAngularThresholdSq = FMath::Square(SleepAngularThreshold);
			Node->SleepCounterThreshold = SleepCounterThreshold;
			Node->WakeLinearThresholdSq = FMath::Square(WakeLinearThreshold);
			Node->WakeAngularThresholdSq = FMath::Square(WakeAngularThreshold);
			Node->DisableLinearThresholdSq = FMath::Square(DisableLinearThreshold);
			Node->DisableAngularThresholdSq = FMath::Square(DisableAngularThreshold);
		}
	}

	void FPBDIslandManager::UpdateGraphNode(FPBDIslandParticle* Node)
	{
		check(Node != nullptr);

		const bool bWasDynamic = Node->Flags.bIsDynamic;
		const bool bWasMoving = Node->Flags.bIsMoving;
		const bool bWasSleeping = Node->Flags.bIsSleeping;
		const bool bIsDynamic = IsParticleDynamic(Node->Particle);
		const bool bIsMoving = IsParticleMoving(Node->Particle);
		const bool bIsSleeping = IsParticleSleeping(Node->Particle);
		const bool bNeedsResim = ParticlesIslandNeedsResim(Node->Particle);

		Node->Flags.bIsDynamic = bIsDynamic;
		Node->Flags.bIsMoving = bIsMoving;
		Node->Flags.bIsSleeping = bIsSleeping;
		Node->Flags.bNeedsResim = bNeedsResim;

		UpdateIslandPartialSleepAllowed(Node->Island, Node->Particle);
		UpdateIslandHasSleepingNode(Node->Island, Node->Particle);

		// Did we change between dynamic and kinematic?
		if (bIsDynamic != bWasDynamic)
		{
			// If we have turned dynamic, we need to merge all islands we are in
			// and update our sleep settings (we don't initialize sleep settings for kinematics)
			if (bIsDynamic)
			{
				UpdateGraphNodeSleepSettings(Node);

				for (FPBDIslandConstraint* Edge : Node->Edges)
				{
					// Merge or assign node islands
					MergeNodeIslands(Edge->Nodes[0], Edge->Nodes[1]);
				}
			}

			// If we have turned kinematic, we need to be removed from our island
			// because kinematics are not directly tracked in islands
			if (!bIsDynamic)
			{
				RemoveNodeFromIsland(Node);
			}

			// Update the edge because the particle type has changed
			// NOTE: reverse iteration since we may remove edges as we go
			for (int32 EdgeIndex = Node->Edges.Num() - 1; EdgeIndex >= 0; --EdgeIndex)
			{
				UpdateGraphEdge(Node->Edges[EdgeIndex]);
			}
		}

		// If the particle changed, we need to check the island's sleep state
		// This is to cover the case where any particle is woken, or all particles are manually slept
		if (bIsDynamic && (!bWasDynamic || (bIsSleeping != bWasSleeping) || bIsMoving))
		{
			if (CVars::bChaosSolverPartialIslandSleep && CVars::bChaosSolverWakeOnParticleStateChanged)
			{
				// When do we want to force the entire island to wake up for at least 1 step? (partial island sleeping only)
				// Case 1) Switch from non-moving kinematic to non-sleeping dynamic OR
				// Case 2) Switch dynamic particle from sleeping to not-sleeping
				const bool bSwitchStaticKinematicToAwakeDynamic = (!bWasDynamic && !bWasMoving) && (bIsDynamic && !bIsSleeping); // Case 1
				const bool bSwitchSleepingDynamicToAwake = (bIsDynamic && bWasSleeping && !bIsSleeping); // Case 2
				const bool bIsSleepAllowed = (bSwitchStaticKinematicToAwakeDynamic || bSwitchSleepingDynamicToAwake) ? false : true;
				EnqueueIslandCheckSleep(Node->Island, bIsSleepAllowed);
			}
			else if (CVars::bChaosSolverPartialIslandSleep && !CVars::bChaosSolverWakeOnParticleStateChanged)
			{
				// Only set bWasUpdated flag if it isn't set yet.
				const bool bIsSleepAllowed = true;
				if (!Node->Flags.bWasUpdated)
				{
					// Register wake-up events for updates in the particles (partial island sleeping only)
					// Case 1) Switch from non-moving kinematic to non-sleeping dynamic OR
					// Case 2) Switch dynamic particle from sleeping to not-sleeping
					const bool bSwitchStaticKinematicToAwakeDynamic = (!bWasDynamic && !bWasMoving) && (bIsDynamic && !bIsSleeping); // Case 1
					const bool bSwitchSleepingDynamicToAwake = (bIsDynamic && bWasSleeping && !bIsSleeping); // Case 2
					const bool bTriggerWakeUpEvent = (bSwitchStaticKinematicToAwakeDynamic || bSwitchSleepingDynamicToAwake);

					Node->Flags.bWasUpdated = bTriggerWakeUpEvent ? true : false;
				}
				EnqueueIslandCheckSleep(Node->Island, bIsSleepAllowed);
			}
			else
			{
				const bool bIsSleepAllowed = true;
				EnqueueIslandCheckSleep(Node->Island, bIsSleepAllowed);
			}
		}

		// If we have a moving kinematic its island(s) cannot sleep
		if (!bIsDynamic && bIsMoving)
		{
			const bool bIsSleepAllowed = false;
			for (FPBDIslandConstraint* Edge : Node->Edges)
			{
				EnqueueIslandCheckSleep(Edge->Island, bIsSleepAllowed);
			}
		}
	}

	void FPBDIslandManager::UpdateGraphEdge(FPBDIslandConstraint* Edge)
	{
		const bool bIsDynamic0 = (Edge->Nodes[0] != nullptr) && Edge->Nodes[0]->Flags.bIsDynamic;
		const bool bIsDynamic1 = (Edge->Nodes[1] != nullptr) && Edge->Nodes[1]->Flags.bIsDynamic;
		if (!bIsDynamic0 && !bIsDynamic1)
		{
			// If we get here, both particles in the constraint are now kinematic and
			// should have already been removed from the island (see FPBDIslandManager::UpdateGraphNode)
			check((Edge->Nodes[0] == nullptr) || (Edge->Nodes[0]->Island == nullptr));
			check((Edge->Nodes[1] == nullptr) || (Edge->Nodes[1]->Island == nullptr));

			// Remove the edge from its island
			RemoveEdgeFromIsland(Edge);

			// Destroy the edge
			DestroyGraphEdge(Edge);
		}
		else
		{
			// We should be in an island already if at least one of the nodes is dynamic
			check(Edge->Island != nullptr);
		}
	}

	FPBDIsland* FPBDIslandManager::CreateIsland()
	{
		FPBDIsland* Island = Islands.Alloc();
		Island->ContainerEdges.SetNum(GetNumConstraintContainers());
		return Island;
	}

	void FPBDIslandManager::DestroyIsland(FPBDIsland* Island)
	{
		if (Island != nullptr)
		{
			check(Island->Nodes.IsEmpty());
			check(Island->NumEdges == 0);
			check(Island->MergeSetIslandIndex == INDEX_NONE);

			Islands.Free(Island);
		}
	}

	void FPBDIslandManager::AssignNodeIsland(FPBDIslandParticle* Node)
	{
		check(Node != nullptr);
		check(Node->Island == nullptr);

		FPBDIsland* Island = CreateIsland();

		AddNodeToIsland(Node, Island);
	}

	void FPBDIslandManager::AssignEdgeIsland(FPBDIslandConstraint* Edge)
	{
		check(Edge != nullptr);
		check(Edge->Island == nullptr);

		FPBDIsland* Island = MergeNodeIslands(Edge->Nodes[0], Edge->Nodes[1]);

		AddEdgeToIsland(Edge, Island);
	}

	void FPBDIslandManager::AddNodeToIsland(FPBDIslandParticle* Node, FPBDIsland* Island)
	{
		check(Island != nullptr);

		if (Node != nullptr)
		{
			// Only dynamic particles are kept in the island's node array.
			// Kinematics may be in many islands (we need to check the node's edges to visit them).
			if (Node->Flags.bIsDynamic)
			{
				UE_MT_SCOPED_WRITE_ACCESS(LocalNodeAccessDetector);
				UE_MT_SCOPED_WRITE_ACCESS(LocalIslandAccessDetector);
				Node->Island = Island;
				Node->IslandArrayIndex = Island->Nodes.Add(Node);
				Island->SetIterationSettings(FIterationSettings::Merge(Island->GetIterationSettings(), Node->GetIterationSettings()));
			}

			Island->Flags.bItemsAdded = true;

			const bool bIsSleepAllowed = true;
			EnqueueIslandCheckSleep(Island, bIsSleepAllowed);
			UpdateIslandPartialSleepAllowed(Island, Node->Particle);
			UpdateIslandHasSleepingNode(Island, Node->Particle);
		}
	}

	void FPBDIslandManager::RemoveNodeFromIsland(FPBDIslandParticle* Node)
	{
		if ((Node != nullptr) && (Node->Island != nullptr))
		{
			FPBDIsland* Island = Node->Island;
			const int32 ArrayIndex = Node->IslandArrayIndex;
			check(Island->Nodes[ArrayIndex] == Node);

			UE_MT_SCOPED_WRITE_ACCESS(LocalIslandAccessDetector);
			UE_MT_SCOPED_WRITE_ACCESS(LocalNodeAccessDetector);

			Island->Nodes.RemoveAtSwap(ArrayIndex, EAllowShrinking::No);
			if (ArrayIndex < Island->Nodes.Num())
			{
				Island->Nodes[ArrayIndex]->IslandArrayIndex = ArrayIndex;
			}

			Island->Flags.bItemsRemoved = true;

			Node->Island = nullptr;
			Node->IslandArrayIndex = INDEX_NONE;
		}
	}

	void FPBDIslandManager::DestroyIslandNodes(FPBDIsland* Island)
	{
		// Destroy any nodes left in the island
		for (int32 IslandNodeIndex = Island->Nodes.Num() - 1; IslandNodeIndex >= 0; --IslandNodeIndex)
		{
			FPBDIslandParticle* Node = Island->Nodes[IslandNodeIndex];
			check(Node->Edges.IsEmpty());

			// If there was a dynamic particle left in the island, we need to transfer the island 
			// sleep state to it for use in ProcessParticlesSleep()
			// @todo(chaos): not great - can we clean this up?
			if (FPBDRigidParticleHandle* Rigid = Node->GetParticle()->CastToRigidParticle())
			{
				Rigid->SetSleepCounter(int8(Island->SleepCounter));
			}

			RemoveNodeFromIsland(Node);
			DestroyGraphNode(Node);
		}
	}

	void FPBDIslandManager::AddEdgeToIsland(FPBDIslandConstraint* Edge, FPBDIsland* Island)
	{
		check(Edge != nullptr);
		check(Island != nullptr);

		UE_MT_SCOPED_WRITE_ACCESS(LocalEdgeAccessDetector);
		UE_MT_SCOPED_WRITE_ACCESS(LocalIslandAccessDetector);

		const int32 ContainerIndex = Edge->ContainerIndex;
		Edge->IslandArrayIndex = Island->ContainerEdges[ContainerIndex].Add(Edge);
		Edge->Island = Island;

		Island->Flags.bItemsAdded = true;
		Island->NumEdges = Island->NumEdges + 1;

		const bool bIsSleepAllowed = true;
		EnqueueIslandCheckSleep(Island, bIsSleepAllowed);
	}

	void FPBDIslandManager::RemoveEdgeFromIsland(FPBDIslandConstraint* Edge)
	{
		check(Edge != nullptr);

		if (Edge->Island != nullptr)
		{
			FPBDIsland* Island = Edge->Island;

			const int32 ContainerIndex = Edge->ContainerIndex;
			const int32 EdgeIndex = Edge->IslandArrayIndex;
			check(Island->ContainerEdges[ContainerIndex][EdgeIndex] == Edge);

			UE_MT_SCOPED_WRITE_ACCESS(LocalEdgeAccessDetector);
			UE_MT_SCOPED_WRITE_ACCESS(LocalIslandAccessDetector);

			Island->ContainerEdges[ContainerIndex].RemoveAtSwap(EdgeIndex, EAllowShrinking::No);
			if (EdgeIndex < Island->ContainerEdges[ContainerIndex].Num())
			{
				Island->ContainerEdges[ContainerIndex][EdgeIndex]->IslandArrayIndex = EdgeIndex;
			}

			Island->Flags.bItemsRemoved = true;
			Island->NumEdges = Island->NumEdges - 1;

			Edge->Island = nullptr;
			Edge->IslandArrayIndex = INDEX_NONE;
		}
	}

	void FPBDIslandManager::WakeNodeIslands(const FPBDIslandParticle* Node)
	{
		check(Node != nullptr);

		// We want to wake up this tick, regardless of sleep thresholds etc
		const bool bIsSleepAllowed = false;

		// Dynamic nodes are in one island, kinematic nodes may be in many
		// and we must vist their edges to discover them
		if (Node->Flags.bIsDynamic)
		{
			EnqueueIslandCheckSleep(Node->Island, bIsSleepAllowed);
			UpdateIslandPartialSleepAllowed(Node->Island, Node->Particle);
		}
		else
		{
			for (FPBDIslandConstraint* Edge : Node->Edges)
			{
				EnqueueIslandCheckSleep(Edge->Island, bIsSleepAllowed);
				UpdateIslandPartialSleepAllowed(Edge->Island, Node->Particle);
			}
		}
	}

	void FPBDIslandManager::EnqueueIslandCheckSleep(FPBDIsland* Island, const bool bIsSleepAllowed)
	{
		if (Island != nullptr)
		{
			Island->Flags.bCheckSleep = true;
			Island->Flags.bIsSleepAllowed = (Island->Flags.bIsSleepAllowed != 0) && bIsSleepAllowed;
		}
	}

	void FPBDIslandManager::UpdateIslandPartialSleepAllowed(FPBDIsland* Island, const FGeometryParticleHandle* Particle)
	{
		// All particles in the island need to allow partial sleep for the island to allow partial sleep.
		const FPBDRigidParticleHandle* Rigid = Particle->CastToRigidParticle();
		if (Island != nullptr && Rigid != nullptr)
		{
			Island->Flags.bIsPartialSleepAllowedInAllNodes = (Island->Flags.bIsPartialSleepAllowedInAllNodes != 0) && Rigid->PartialIslandSleepAllowed();
		}
	}

	void FPBDIslandManager::ResetIslandPartialSleepAllowed()
	{
		for (FPBDIsland* Island : Islands)
		{
			Island->Flags.bIsPartialSleepAllowedInAllNodes = true;
		}
	}

	void FPBDIslandManager::UpdateIslandHasSleepingNode(FPBDIsland* Island, const FGeometryParticleHandle* Particle)
	{
		const FPBDRigidParticleHandle* Rigid = Particle->CastToRigidParticle();
		if (Island != nullptr && Rigid != nullptr)
		{
			Island->Flags.bHasSleepingNode = (!!Island->Flags.bHasSleepingNode) || Rigid->IsSleeping();
		}
	}

	FPBDIsland* FPBDIslandManager::MergeNodeIslands(FPBDIslandParticle* Node0, FPBDIslandParticle* Node1)
	{
		// NOTE: We only register the islands for merging here. The actual merge takes place later in ProcessMerges.

		FPBDIsland* Island0 = (Node0 != nullptr) ? Node0->Island : nullptr;
		FPBDIsland* Island1 = (Node1 != nullptr) ? Node1->Island : nullptr;

		// Don't try to merge island into self
		if ((Island0 != nullptr) && (Island0 == Island1))
		{
			return Island0;
		}

		FPBDIsland* Island = nullptr;
		if ((Island0 != nullptr) && (Island1 != nullptr))
		{
			Island = MergeIslands(Island0, Island1);
		}
		else
		{
			if (Island0 != nullptr) // && (Island1 == nullptr)
			{
				Island = Island0;
				AddNodeToIsland(Node1, Island);
			}
			else if (Island1 != nullptr) // && (Island0 == nullptr)
			{
				Island = Island1;
				AddNodeToIsland(Node0, Island);
			}
			else // ((Island0 == nullptr) && (Island1 == nullptr))
			{
				// NOTE: Islands are created in the sleeping state and will be set to
				// awake if either node is moving when we add them
				Island = CreateIsland();
				AddNodeToIsland(Node0, Island);
				AddNodeToIsland(Node1, Island);
			}
		}

		return Island;
	}

	FPBDIsland* FPBDIslandManager::MergeIslands(FPBDIsland* Island0, FPBDIsland* Island1)
	{
		// NOTE: We only regsiters the islands for merging here. The actual merge takes place later in ProcessMerges.
		check(Island0 != nullptr);
		check(Island1 != nullptr);

		if (Island0 == Island1)
		{
			return Island0;
		}

		// Add the islands to a merge set, which keeps track of all the islands that have been merged into each other
		// If both islands are already assigned to a merge set, merge the merge sets (if different)
		// Otherwise add the island to the existing or created merge set
		FPBDIslandMergeSet* MergeSet0 = Island0->MergeSet;
		FPBDIslandMergeSet* MergeSet1 = Island1->MergeSet;
		if ((MergeSet0 == nullptr) || (MergeSet0 != MergeSet1))
		{
			if ((MergeSet0 != nullptr) && (MergeSet1 != nullptr))
			{
				CombineMergeSets(MergeSet0, MergeSet1);
			}
			else if (MergeSet0 != nullptr) // && (MergeSet1 == nullptr)
			{
				AddIslandToMergeSet(Island1, MergeSet0);
			}
			else if (MergeSet1 != nullptr) // && (MergeSet0 == nullptr)
			{
				AddIslandToMergeSet(Island0, MergeSet1);
			}
			else // (MergeSet0 == nullptr) && (MergSet1 == nullptr)
			{
				CreateMergeSet(Island0, Island1);
			}
		}

		return (Island0->NumEdges >= Island1->NumEdges) ? Island0 : Island1;
	}

	FPBDIslandMergeSet* FPBDIslandManager::CreateMergeSet(FPBDIsland* Island0, FPBDIsland* Island1)
	{
		FPBDIslandMergeSet* MergeSet = MergeSets.Alloc();

		AddIslandToMergeSet(Island0, MergeSet);
		AddIslandToMergeSet(Island1, MergeSet);

		return MergeSet;
	}

	void FPBDIslandManager::DestroyMergeSet(FPBDIslandMergeSet* MergeSet)
	{
		if (MergeSet != nullptr)
		{
			MergeSets.Free(MergeSet);
		}
	}

	void FPBDIslandManager::AddIslandToMergeSet(FPBDIsland* Island, FPBDIslandMergeSet* MergeSet)
	{
		check(Island != nullptr);
		check(Island->MergeSet == nullptr);

		// Bind the island to the merge set
		Island->MergeSet = MergeSet;
		Island->MergeSetIslandIndex = MergeSet->Islands.Add(Island);
		MergeSet->NumEdges += Island->NumEdges;
	}

	void FPBDIslandManager::RemoveIslandFromMergeSet(FPBDIsland* Island)
	{
		check(Island != nullptr);

		if (Island->MergeSet != nullptr)
		{
			FPBDIslandMergeSet* MergeSet = Island->MergeSet;

			// Remove from the list of islands to merge
			const int32 IslandIndex = Island->MergeSetIslandIndex;
			MergeSet->Islands.RemoveAtSwap(IslandIndex, EAllowShrinking::No);
			if (IslandIndex < MergeSet->Islands.Num())
			{
				MergeSet->Islands[IslandIndex]->MergeSetIslandIndex = IslandIndex;
			}

			// Update edge count
			MergeSet->NumEdges -= Island->NumEdges;

			Island->MergeSet = nullptr;
			Island->MergeSetIslandIndex = INDEX_NONE;
		}
	}

	FPBDIslandMergeSet* FPBDIslandManager::CombineMergeSets(FPBDIslandMergeSet* MergeSetParent, FPBDIslandMergeSet* MergeSetChild)
	{
		check(MergeSetParent != nullptr);
		check(MergeSetChild != nullptr);

		if (MergeSetParent == MergeSetChild)
		{
			return MergeSetParent;
		}

		// Select the largest set as the merge parent
		// (we should merge into the set with the largest island, but this should be good enough)
		if (MergeSetParent->NumEdges < MergeSetChild->NumEdges)
		{
			Swap(MergeSetParent, MergeSetChild);
		}

		// Tell all islands about their new merge set
		int32 ChildIslandIndex = MergeSetParent->Islands.Num();
		for (FPBDIsland* Island : MergeSetChild->Islands)
		{
			Island->MergeSet = MergeSetParent;
			Island->MergeSetIslandIndex = ChildIslandIndex++;
		}

		// Add the islands to their new merge set
		MergeSetParent->Islands.Append(MoveTemp(MergeSetChild->Islands));

		// Update edge count
		MergeSetParent->NumEdges += MergeSetChild->NumEdges;

		// Destroy the emptied merge set
		DestroyMergeSet(MergeSetChild);

		return MergeSetParent;
	}

	FPBDIsland* FPBDIslandManager::GetMergeSetParentIsland(FPBDIslandMergeSet* MergeSet, int32& OutNumNodes, const TArrayView<int32>& OutNumContainerEdges)
	{
		check(MergeSet != nullptr);

		// Initialize outputs
		FPBDIsland* LargestIsland = nullptr;
		OutNumNodes = 0;
		for (int32 ContainerIndex = 0; ContainerIndex < OutNumContainerEdges.Num(); ++ContainerIndex)
		{
			OutNumContainerEdges[ContainerIndex] = 0;
		}

		// Find the largest island and count the nodes and edges from all islands
		for (FPBDIsland* Island : MergeSet->Islands)
		{
			check(Island != nullptr);

			if ((LargestIsland == nullptr) || (Island->NumEdges > LargestIsland->NumEdges))
			{
				LargestIsland = Island;
			}

			OutNumNodes += Island->Nodes.Num();

			for (int32 ContainerIndex = 0; ContainerIndex < Island->ContainerEdges.Num(); ++ContainerIndex)
			{
				OutNumContainerEdges[ContainerIndex] += Island->ContainerEdges[ContainerIndex].Num();
			}
		}

		return LargestIsland;
	}

	void FPBDIslandManager::ProcessIslands()
	{
		ProcessMerges();
		ProcessSplits();
		ProcessWakes();
		AssignLevels();
		FinalizeIslands();
	}

	void FPBDIslandManager::ProcessMerges()
	{
		SCOPE_CYCLE_COUNTER(STAT_IslandManager_MergeIslands);

		int32 NumNodes;
		TArray<int32> NumContainerEdges;
		NumContainerEdges.SetNumUninitialized(GetNumConstraintContainers());

		// Could go wide here...
		for (FPBDIslandMergeSet* MergeSet : MergeSets)
		{
			// Find the island that we will merge into (the biggest one) and count the nodes and edges
			if (FPBDIsland* ParentIsland = GetMergeSetParentIsland(MergeSet, NumNodes, MakeArrayView(NumContainerEdges)))
			{
				// Reserve space for all the nodes and edges
				ParentIsland->Nodes.Reserve(NumNodes);
				for (int32 ContainerIndex = 0; ContainerIndex < NumContainerEdges.Num(); ++ContainerIndex)
				{
					ParentIsland->ContainerEdges[ContainerIndex].Reserve(NumContainerEdges[ContainerIndex]);
				}

				// Merge each child island into the parent island
				for (FPBDIsland* ChildIsland : MergeSet->Islands)
				{
					// Reset the merge set ready for next tick. We only really need to do this on the island
					// that survives the merge, but it helps with error tracking to do it for all (see DestroyIsland)
					ChildIsland->MergeSet = nullptr;
					ChildIsland->MergeSetIslandIndex = INDEX_NONE;

					ProcessIslandMerge(ParentIsland, ChildIsland);
				}
			}
		}

		MergeSets.Reset();
	}

	void FPBDIslandManager::ProcessIslandMerge(FPBDIsland* ParentIsland, FPBDIsland* ChildIsland)
	{
		if (ParentIsland != ChildIsland)
		{
			// Combine island state
			ParentIsland->SleepCounter = FMath::Min(ParentIsland->SleepCounter, ChildIsland->SleepCounter);
			ParentIsland->Flags.bItemsAdded = true;
			ParentIsland->Flags.bItemsRemoved = !!ParentIsland->Flags.bItemsRemoved || !!ChildIsland->Flags.bItemsRemoved;
			ParentIsland->Flags.bIsSleepAllowed = !!ParentIsland->Flags.bIsSleepAllowed && !!ChildIsland->Flags.bIsSleepAllowed;
			ParentIsland->Flags.bIsPartialSleepAllowedInAllNodes = !!ParentIsland->Flags.bIsPartialSleepAllowedInAllNodes && !!ChildIsland->Flags.bIsPartialSleepAllowedInAllNodes;
			ParentIsland->Flags.bHasSleepingNode = !!ParentIsland->Flags.bHasSleepingNode || !!ChildIsland->Flags.bHasSleepingNode;
			ParentIsland->Flags.bIsSleeping = !!ParentIsland->Flags.bIsSleeping && !!ChildIsland->Flags.bIsSleeping;
			ParentIsland->Flags.bWasSleeping = !!ParentIsland->Flags.bWasSleeping || !!ChildIsland->Flags.bWasSleeping;
			ParentIsland->Flags.bCheckSleep = !!ParentIsland->Flags.bCheckSleep || !!ChildIsland->Flags.bCheckSleep;
			ParentIsland->Flags.bNeedsResim = false;	// Calculated in FinalizeIslands

			// Tell all the nodes which island they belong to and what index they will have (after we move the elements)
			int32 NextIslandArrayIndex = ParentIsland->Nodes.Num();
			for (FPBDIslandParticle* Node : ChildIsland->Nodes)
			{
				if (Node->Flags.bIsDynamic)
				{
					Node->Island = ParentIsland;
					Node->IslandArrayIndex = NextIslandArrayIndex++;
					ParentIsland->SetIterationSettings(FIterationSettings::Merge(ParentIsland->GetIterationSettings(), Node->GetIterationSettings()));
				}
			}

			// Tell all the edges which island they belong to and what index they will have (after we move the elements)
			for (int32 ContainerIndex = 0; ContainerIndex < ChildIsland->ContainerEdges.Num(); ++ContainerIndex)
			{
				int32 NextContainerEdgeIndex = ParentIsland->ContainerEdges[ContainerIndex].Num();
				for (FPBDIslandConstraint* Edge : ChildIsland->ContainerEdges[ContainerIndex])
				{
					Edge->Island = ParentIsland;
					Edge->IslandArrayIndex = NextContainerEdgeIndex++;
				}
			}

			// Move the nodes and edges to the parent
			ParentIsland->Nodes.Append(MoveTemp(ChildIsland->Nodes));
			ChildIsland->Nodes.Reset();

			for (int32 ContainerIndex = 0; ContainerIndex < ChildIsland->ContainerEdges.Num(); ++ContainerIndex)
			{
				ParentIsland->NumEdges += ChildIsland->ContainerEdges[ContainerIndex].Num();
				ParentIsland->ContainerEdges[ContainerIndex].Append(MoveTemp(ChildIsland->ContainerEdges[ContainerIndex]));

				ChildIsland->ContainerEdges[ContainerIndex].Reset();
			}
			ChildIsland->NumEdges = 0;

			// Destroy the child island
			DestroyIsland(ChildIsland);
		}
	}

	void FPBDIslandManager::ProcessSplits()
	{
		SCOPE_CYCLE_COUNTER(STAT_IslandManager_SplitIslands);

		// Could go wide here...(though a bit complicated since we create and destroy islands)
		for (int32 IslandIndex = Islands.Num() - 1; IslandIndex >= 0; --IslandIndex)
		{
			FPBDIsland* Island = Islands[IslandIndex];
			if (Island->Flags.bItemsRemoved)
			{
				ProcessIslandSplits(Island);
			}
		}
	}

	void FPBDIslandManager::ProcessIslandSplits(FPBDIsland* Island)
	{
		check(Island != nullptr);
		check(Island->Flags.bItemsRemoved);

		// Reset split tracking indicesRemo
		for (FPBDIslandParticle* Node : Island->Nodes)
		{
			Node->Island = nullptr;
			Node->IslandArrayIndex = INDEX_NONE;
		}
		for (int32 ContainerIndex = 0; ContainerIndex < Island->ContainerEdges.Num(); ++ContainerIndex)
		{
			for (FPBDIslandConstraint* Edge : Island->ContainerEdges[ContainerIndex])
			{
				Edge->Island = nullptr;
				Edge->IslandArrayIndex = INDEX_NONE;
			}
		}
		const int32 VisitEpoch = GetNextVisitEpoch();

		// Extract the nodes and reset the current island. We rebuild below.
		TArray<FPBDIslandParticle*> IslandNodes;
		Swap(IslandNodes, Island->Nodes);
		for (int32 ContainerIndex = 0; ContainerIndex < Island->ContainerEdges.Num(); ++ContainerIndex)
		{
			Island->ContainerEdges[ContainerIndex].Reset();
		}
		Island->NumEdges = 0;

		// The next island to add nodes/edges to
		FPBDIsland* CurrentIsland = Island;

		// Reset partial sleep allowed flag
		// Will be updated when the island is repopulated (call to UpdateIslandPartialSleepAllowed) 
		CurrentIsland->Flags.bIsPartialSleepAllowedInAllNodes = true;

		// Reset has sleeping node flag
		// Will be updated when the island is repopulated (call to UpdateIslandHasSleepingNodes) 
		CurrentIsland->Flags.bHasSleepingNode = false;

		// Visit all nodes and put connected nodes into an island
		TArray<FPBDIslandParticle*> NodeQueue;
		NodeQueue.Reserve(Island->Nodes.Num());

		// We need to remember this value because it gets overwritten when
		// rebuilding the islands below (see AddNodeToIsland)
		const bool bCheckSleep = Island->Flags.bCheckSleep;

		for (FPBDIslandParticle* RootNode : IslandNodes) //-V1078 7.39 Swap Bug
		{
			// If we haven't seen this node yet and it is dynamic, start forming a new island
			if ((RootNode->VisitEpoch != VisitEpoch) && RootNode->Flags.bIsDynamic)
			{
				// Create an island if we need one
				if (CurrentIsland == nullptr)
				{
					CurrentIsland = CreateIsland();
				}

				// Add the root node to the island and visit all connected nodes
				AddNodeToIsland(RootNode, CurrentIsland);
				UpdateIslandPartialSleepAllowed(CurrentIsland, RootNode->Particle);
				UpdateIslandHasSleepingNode(CurrentIsland, RootNode->Particle);
				NodeQueue.Add(RootNode);
				RootNode->VisitEpoch = VisitEpoch;

				// Populate the island with all connected nodes and edges
				while (!NodeQueue.IsEmpty())
				{
					FPBDIslandParticle* NextNode = NodeQueue.Pop(EAllowShrinking::No);

					// Visit all the edges connected to the current node
					for (FPBDIslandConstraint* Edge : NextNode->Edges)
					{
						if (Edge->VisitEpoch != VisitEpoch)
						{
							// Add the edge to the current island
							AddEdgeToIsland(Edge, CurrentIsland);
							Edge->VisitEpoch = VisitEpoch;

							// Queue the other connected node for processing
							FPBDIslandParticle* EdgeOtherNode = (NextNode == Edge->Nodes[0]) ? Edge->Nodes[1] : Edge->Nodes[0];
							if ((EdgeOtherNode != nullptr) && EdgeOtherNode->Flags.bIsDynamic && (EdgeOtherNode->VisitEpoch != VisitEpoch))
							{
								AddNodeToIsland(EdgeOtherNode, CurrentIsland);
								UpdateIslandPartialSleepAllowed(CurrentIsland, EdgeOtherNode->Particle);
								UpdateIslandHasSleepingNode(CurrentIsland, EdgeOtherNode->Particle);
								NodeQueue.Add(EdgeOtherNode);
								EdgeOtherNode->VisitEpoch = VisitEpoch;
							}
						}
					}
				}

				// Set the island state
				CurrentIsland->SleepCounter = Island->SleepCounter;
				CurrentIsland->Flags.bItemsAdded = true;
				CurrentIsland->Flags.bItemsRemoved = true;
				CurrentIsland->Flags.bIsSleepAllowed = Island->Flags.bIsSleepAllowed;
				CurrentIsland->Flags.bIsSleeping = Island->Flags.bIsSleeping;
				CurrentIsland->Flags.bWasSleeping = Island->Flags.bWasSleeping;
				CurrentIsland->Flags.bCheckSleep = bCheckSleep;
				CurrentIsland->Flags.bNeedsResim = Island->Flags.bNeedsResim;

				// We are done with this island
				CurrentIsland = nullptr;
			}
		}
	}

	void FPBDIslandManager::AssignLevels()
	{
		SCOPE_CYCLE_COUNTER(STAT_IslandManager_AssignLevels);

		// Could go wide here...
		for (FPBDIsland* Island : Islands)
		{
			// Levels will only change when awake and when something has been added or removed
			if (!Island->Flags.bIsSleeping && (!!Island->Flags.bItemsAdded || !!Island->Flags.bItemsRemoved))
			{
				// We only need to assign levels if shock propagation or partial island sleeping is enabled (but we always sort)
				if (ShouldAssignLevels(Island))
				{
					AssignIslandLevels(Island);
				}
				SortIslandEdges(Island);
			}
		}
	}

	void FPBDIslandManager::AssignIslandLevels(FPBDIsland* Island)
	{
		// Reset levels: 0 means unassigned; 1 means directly attached to a kinematic
		// (default 0 is so that the SortKey work even when we don't assign levels)
		for (FPBDIslandParticle* Node : Island->Nodes)
		{
			Node->Level = 0;
		}
		for (int32 ContainerIndex = 0; ContainerIndex < Island->ContainerEdges.Num(); ++ContainerIndex)
		{
			for (FPBDIslandConstraint* Edge : Island->ContainerEdges[ContainerIndex])
			{
				Edge->Level = 0;
			}
		}

		if (Island->NumEdges < 2)
		{
			return;
		}

		TArray<FPBDIslandParticle*> NodeQueue;
		NodeQueue.Reserve(Island->Nodes.Num());

		// Initialize edge levels and populate the queue with all the level-0 nodes
		for (int32 ContainerIndex = 0; ContainerIndex < Island->ContainerEdges.Num(); ++ContainerIndex)
		{
			for (FPBDIslandConstraint* Edge : Island->ContainerEdges[ContainerIndex])
			{
				check((Edge->Nodes[0] != nullptr) || (Edge->Nodes[1] != nullptr));
				const bool bIsDynamic0 = ((Edge->Nodes[0] != nullptr) && Edge->Nodes[0]->Flags.bIsDynamic);
				const bool bIsDynamic1 = ((Edge->Nodes[1] != nullptr) && Edge->Nodes[1]->Flags.bIsDynamic);

				if (!bIsDynamic0 && (Edge->Nodes[1]->Level == 0))
				{
					Edge->Level = 1;
					Edge->Nodes[1]->Level = 1;
					NodeQueue.Add(Edge->Nodes[1]);
				}
				if (!bIsDynamic1 && (Edge->Nodes[0]->Level == 0))
				{
					Edge->Level = 1;
					Edge->Nodes[0]->Level = 1;
					NodeQueue.Add(Edge->Nodes[0]);
				}
			}
		}

		// Breadth-first visit all nodes, assign a level to the connected node, and enqueue it
		for (int32 NodeIndex = 0; NodeIndex < NodeQueue.Num(); ++NodeIndex)
		{
			FPBDIslandParticle* Node = NodeQueue[NodeIndex];
			for (FPBDIslandConstraint* Edge : Node->Edges)
			{
				if (Edge->Level == 0)
				{
					Edge->Level = Node->Level + 1;

					FPBDIslandParticle* OtherNode = (Node == Edge->Nodes[0]) ? Edge->Nodes[1] : Edge->Nodes[0];
					if ((OtherNode != nullptr) && OtherNode->Flags.bIsDynamic && (OtherNode->Level == 0))
					{
						OtherNode->Level = Node->Level + 1;
						NodeQueue.Add(OtherNode);
					}
				}
			}
		}
	}

	void FPBDIslandManager::SortIslandEdges(FPBDIsland* Island)
	{
		// Sort the constraints based on level, and then add order (within each level)
		for (int32 ContainerIndex = 0; ContainerIndex < Island->ContainerEdges.Num(); ++ContainerIndex)
		{
			Island->ContainerEdges[ContainerIndex].Sort(
				[this](const FPBDIslandConstraint& L, const FPBDIslandConstraint& R)
				{
					return L.GetSortKey() < R.GetSortKey();
				});

			int32 IslandArrayIndex = 0;
			for (FPBDIslandConstraint* Edge : Island->ContainerEdges[ContainerIndex])
			{
				Edge->IslandArrayIndex = IslandArrayIndex++;
			}
		}
	}

	void FPBDIslandManager::RandomizeIslandEdges(FPBDIsland* Island)
	{
		for (int32 ContainerIndex = 0; ContainerIndex < Island->ContainerEdges.Num(); ++ContainerIndex)
		{
			TArray<FPBDIslandConstraint*>& ContainerConstraints = Island->ContainerEdges[ContainerIndex];
			for (int32 SwapIndex = 0; SwapIndex < ContainerConstraints.Num(); ++SwapIndex)
			{
				const int32 IndexA = FMath::RandRange(0, ContainerConstraints.Num() - 1);
				const int32 IndexB = FMath::RandRange(0, ContainerConstraints.Num() - 1);
				Swap(ContainerConstraints[IndexA], ContainerConstraints[IndexB]);
				ContainerConstraints[IndexA]->IslandArrayIndex = IndexA;
				ContainerConstraints[IndexB]->IslandArrayIndex = IndexB;
			}
		}
	}

	void FPBDIslandManager::ProcessWakes()
	{
		for (FPBDIsland* Island : Islands)
		{
			if (Island->Flags.bCheckSleep)
			{
				// See if we should sleep or wake based on particle state and whether we are allowed to sleep
				// (islands with a moving kinematic are not allowed to sleep)
				bool bIsSleeping = Island->Flags.bIsSleepAllowed;
				if (bIsSleeping)
				{
					for (const FPBDIslandParticle* Node : Island->Nodes)
					{
						bIsSleeping = bIsSleeping && Node->Flags.bIsSleeping;
						if (!bIsSleeping)
						{
							break;
						}
					}
				}

				if (bIsSleeping != Island->Flags.bIsSleeping)
				{
					Island->Flags.bIsSleeping = bIsSleeping;
					Island->SleepCounter = 0;
				}
			}
		}
	}

	void FPBDIslandManager::FinalizeIslands()
	{
		SCOPE_CYCLE_COUNTER(STAT_IslandManager_Finalize);

		for (int32 IslandIndex = Islands.Num() - 1; IslandIndex >= 0; --IslandIndex)
		{
			FPBDIsland* Island = Islands[IslandIndex];

			// If the island is awake or just changed state, make sure all the particles and constraints agree
			// NOTE: also handles the case where we add to a sleeping island (by checking bCheckSleep)
			// @todo(chaos) : (island sleeping only) we should only need to do this when the state changes... ?
			if (!Island->Flags.bIsSleeping || (Island->Flags.bIsSleeping != Island->Flags.bWasSleeping) || !!Island->Flags.bCheckSleep)
			{
				if (Island->IsPartialSleepEnabled() && !!Island->Flags.bIsSleepAllowed &&
					(CVars::bChaosSolverForcePartialIslandWake || !!Island->Flags.bHasSleepingNode))
				{
					if (!Island->Flags.bIsSleeping)
					{
						ProcessPartialIslandWake(Island, /*bProcessEdgeBasedWake=*/ true);
					}
					SyncConstraintSleepStateToParticles(Island);
				}
				else // For island sleeping to sync the sleep state OR if (!bIsSleepAllowed) to wake up the entire island
				{
					PropagateIslandSleep(Island);
				}
				Island->UpdateSyncState();
			}

			// Remove islands without constraints. NOTE: must come after PropagateIslandSleep
			// so that we wake nodes the have been left on their own after all other nodes were removed.
			if (Island->NumEdges == 0)
			{
				// Destroy nodes that are left in the island
				// NOTE: also copies back any island state needed by the particle (sleep counter)
				DestroyIslandNodes(Island);

				DestroyIsland(Island);
				continue;
			}

			// Reset of the sleep counter if the island is:
			// - Not allowed to sleep this tick
			// - Sleeping, since as soon as it wakes up we could start incrementing the counter
			// - Just woken because we may not have been asleep for a whole frame (this probably makes the previous check unnecessary)
			if (!Island->Flags.bIsSleepAllowed || !!Island->Flags.bIsSleeping || (Island->Flags.bIsSleeping != Island->Flags.bWasSleeping))
			{
				Island->SleepCounter = 0;
			}

			// @todo(chaos): this won't handle updates to per-body iterations after being added to an island, until the island
			// changes for some other reason. Not a big deal but should probably be fixed at some point.
			if (Island->Flags.bItemsAdded || Island->Flags.bItemsRemoved)
			{
				// Compute max of iteration settings by iterating through all the nodes:
				// NOTE: It is valid for any or all iteration counts to be undefined (-1) - this means "use project settings"
				SCOPE_CYCLE_COUNTER(STAT_IslandManager_ComputeIterationSettings);
				if (Chaos::CVars::bChaosSolverComputeIterationSettings)
				{
					// Default to undefined (-1) - use project settings
					FIterationSettings IslandIterations = FIterationSettings();
					for (FPBDIslandParticle* Node : Island->Nodes)
					{
						if (Node->Flags.bIsDynamic)
						{
							IslandIterations = FIterationSettings::Merge(IslandIterations, Node->GetIterationSettings());
						}
					}
					Island->SetIterationSettings(IslandIterations);
				}
			}

			// Randomize constraint order for testing
			if (CVars::bChaosRandomizeConstraintOrder)
			{
				RandomizeIslandEdges(Island);
			}

			Island->Flags.bWasSleeping = Island->Flags.bIsSleeping;
			Island->Flags.bCheckSleep = false;
			Island->Flags.bItemsAdded = false;
			Island->Flags.bItemsRemoved = false;
			Island->ResimFrame = INDEX_NONE;
		}

		if (CVars::bChaosSolverPartialIslandSleep)
		{
			ResetEdgeFlags();
			ResetNodeFlags();
		}

		Validate();
	}

	void FPBDIslandManager::UpdateExplicitSleep()
	{
		// UpdateExplicitSleep should be called after processing physics inputs that may change the sleep/wake status of particles. 
		// It is required so that the Particle/Constraint/Island sleep states are in sync when we Integrate and Detection Collisions.
		// E.g., If we do not do this and a particle was explicitly put to sleep, it will not be integrated and no collisions will 
		// be detected. If that particle is in an island with other awake particles it will be woken immediately but will not have 
		// moved or have any collisions.

		const auto ShouldIslandSleep = [](FPBDIsland* Island) -> bool
			{
				for (FPBDIslandParticle* Node : Island->GetParticles())
				{
					const bool bIsDynamic = IsParticleDynamic(Node->GetParticle());
					const bool bIsSleeping = IsParticleSleeping(Node->GetParticle());
					if (bIsDynamic && !bIsSleeping)
					{
						return false;
					}
				}
				return true;
			};

		// If we have explicitly made some particles go to sleep (as opposed to them naturally sleeping based on low movement)
		// we must check to see if the whole island can go to sleep. This is primarily to address the issue that we do not
		// update collisions for sleeping particles and destroy collisions in awake islands that are not updated this tick.
		for (FPBDIsland* Island : Islands)
		{
			if (!!Island->Flags.bCheckSleep)
			{
				const bool bIslandShouldSleep = !!Island->Flags.bIsSleepAllowed && ShouldIslandSleep(Island);
				if (bIslandShouldSleep != Island->Flags.bIsSleeping)
				{
					Island->Flags.bIsSleeping = bIslandShouldSleep;
					Island->SleepCounter = 0;
				}

				// @todo(Chaos): Island->Flags.bHasSleepingNode is always false here since it's reset in ProcessSleep and will be repopulated in CreateConstraintGraph.
				// Since this code is rarely executed, we tolerate this and just do partial wakes even though it's not necessary (i.e. flag is not used here).
				if (Island->IsPartialSleepEnabled() && CVars::bChaosSolverPartialWakePreIntegration && !!Island->Flags.bIsSleepAllowed)
				{
					if (!Island->Flags.bIsSleeping)
					{
						// Pre-CD call, no new edges to process waking for.
						ProcessPartialIslandWake(Island, /*bProcessEdgeBasedWake=*/ false);
					}
					SyncConstraintSleepStateToParticles(Island);

					// We don't want to reset Node->Flags.bWasUpdated and Node->Flags.bWasTransformSetExplicitly since we need to check them in post-integration waking (FinalizeIslands).
					// Hence, do not call ResetNodeFlags(Island), but reset the Node->Flags.bWasProcessed manually here.
					ResetEdgeFlags(Island);
					for (FPBDIslandParticle* Node : Island->Nodes)
					{
						Node->Flags.bWasProcessed = false;
					}
				}
				else // For island sleeping to sync the sleep state OR if (!bIsSleepAllowed) to wake up the entire island
				{
					// NOTE: we only get here if particle sleep state was changed. We need to ensure that
					// constraint sleep state matches the particle sleep state. Ideally we would only
					// do this if we know that they don't match, but that's hard to determine.
					PropagateIslandSleep(Island);

					if (!Island->Flags.bIsPartialSleepAllowedInAllNodes)
					{
						ResetParticleSleepCounter(Island);
					}
				}

				Island->Flags.bCheckSleep = false;
			}
		}
	}

	void FPBDIslandManager::ProcessSleep(const FRealSingle Dt)
	{
		if (!CVars::bChaosSolverSleepEnabled)
		{
#if CHAOS_DEBUG_DRAW
			// Debugging only: Update velocity smoothing so that debug draw works even if sleeping is disabled
			UpdateAllParticleSleepMetrics(Dt);
#endif
			return;
		}

		// Isolated particles are not kept in any island and need to be handled separately.
		ProcessParticlesSleep(Dt);

		// @todo(chaos): can go wide except for PropagateSleepState
		for (FPBDIsland* Island : Islands)
		{
			if (!Island->Flags.bIsSleeping && !!Island->Flags.bIsSleepAllowed && !Island->Flags.bIsUsingCache)
			{
				if (Island->IsPartialSleepEnabled())
				{
					// Update the sleep state based on particle movement
					// Deactivate non-moving particles
					// Deactivate constraints between sleeping particles
					ProcessPartialIslandSleep(Island, Dt);
				}
				else
				{
					// Update the sleep state based on particle movement etc
					ProcessIslandSleep(Island, Dt);

					if (!!Island->Flags.bIsSleeping)
					{
						PropagateIslandSleep(Island);
					}
				}
			}

			// This gets set to false again next tick if there are moving kinematics in the island
			Island->Flags.bIsSleepAllowed = true;

			Island->Flags.bHasSleepingNode = false; // not valid anymore, repopulated in CreateConstraintGraph
		}
	}

	void FPBDIslandManager::ProcessParticlesSleep(const FRealSingle Dt)
	{
		// Check the sleepiness of particles that are not in any islands.
		// @todo(chaos): this is very expensive because we have to search for a material in GetParticleSleepThresholds.
		// We should probably cache a particle's sleep (and disable) thresholds somewhere.
		TArray<FGeometryParticleHandle*> SleptParticles;
		TArray<FGeometryParticleHandle*> DisabledParticles;
		for (FTransientPBDRigidParticleHandle& Rigid : Particles.GetActiveDynamicMovingKinematicParticlesView())
		{
			if (Rigid.IsDynamic() && !Rigid.IsInConstraintGraph())
			{
				FRealSingle SleepLinearThreshold, SleepAngularThreshold;
				int32 SleepCounterThreshold;
				GetIsolatedParticleSleepThresholds(Rigid.Handle(), PhysicsMaterials, PerParticlePhysicsMaterials, SimMaterials, SleepLinearThreshold, SleepAngularThreshold, SleepCounterThreshold);

				// Check for sleep
				if ((SleepLinearThreshold > 0) || (SleepAngularThreshold > 0))
				{
					// NOTE: We do not use smoothed velocity for isolated particles (smoothed velocity is used to hide
					// minor collision/joint jitter and that won't be present) so we reset it for isolated particles
					InitParticleSleepMetrics(Rigid, Dt);

					int32 SleepCounter = 0;
					if (SleepCounterThreshold < TNumericLimits<int32>::Max())
					{
						// Isolated particles have a max sleep counter of 127 (to reduce counter space in the particle)
						SleepCounterThreshold = FMath::Min(SleepCounterThreshold, TNumericLimits<int8>::Max());

						// Did we exceed the velocity threshold?
						if ((Rigid.VSmooth().SizeSquared() > FMath::Square(SleepLinearThreshold))
							|| (Rigid.WSmooth().SizeSquared() > FMath::Square(SleepAngularThreshold)))
						{
							continue;
						}

						// If we get here we want to sleep
						// Update the counter and sleep if we exceed it
						static_assert(sizeof(decltype(Rigid.SleepCounter())) == 1, "Expected int8 for SleepCounter(). Update clamp below");
						SleepCounter = FMath::Min(int32(Rigid.SleepCounter()) + 1, int32(TNumericLimits<int8>::Max()));
						if (SleepCounter > SleepCounterThreshold)
						{
							SleptParticles.Add(Rigid.Handle());
							SleepCounter = 0;
						}
					}
					Rigid.SetSleepCounter(int8(SleepCounter));
				}
			}
		}

		if (!SleptParticles.IsEmpty())
		{
			Particles.DeactivateParticles(SleptParticles);
		}
	}

	void FPBDIslandManager::ProcessIslandSleep(FPBDIsland* Island, const FRealSingle Dt)
	{
		bool bWithinSleepThreshold = true;
		int32 SleepCounterThreshold = 0;
		int32 MotionlessCount = 0;
		for (FPBDIslandParticle* Node : Island->GetParticles())
		{
			// All zeroes means never sleep/disable
			if ((Node->SleepLinearThresholdSq <= 0) && (Node->SleepAngularThresholdSq <= 0))
			{
				bWithinSleepThreshold = false;
				// NOTE: We will not sleep if there is a particle with non-positive thresholds.
				// But we want to keep iterating to update MotionlessCount.
				continue;
			}

			// Check the particle state against the thresholds
			FPBDRigidParticleHandle* Rigid = Node->GetParticle()->CastToRigidParticle();
			if (Rigid == nullptr)
			{
				continue;
			}

			// The particle SleepType may have changed since the node cached thresholds
			// This is needed because we have API on the particle to SetSleepType which doesn't end up updating thresholds which is done when setting SleepType via FPBDRigidsEvolutionBase::SetParticleSleepType
			if (Rigid->SleepType() == ESleepType::NeverSleep)
			{
				bWithinSleepThreshold = false;
				// NOTE: We will not sleep if there is a never-sleeping particle.
				// But we want to keep iterating to update MotionlessCount.
				continue;
			}

			UpdateParticleSleepMetrics(*Rigid, Dt);

			// Did we exceed the velocity threshold?
			if ((Rigid->VSmooth().SizeSquared() > Node->SleepLinearThresholdSq)
				|| (Rigid->WSmooth().SizeSquared() > Node->SleepAngularThresholdSq))
			{
				bWithinSleepThreshold = false;
				Rigid->SetSleepCounter(int8(0));

				// NOTE: We will not sleep if any particle exceeds the threshold, so we could "break" here.
				// However we still want to update the SleepMetrics for all particles because we want to
				// update the smoothed velocity based on current state, so we must continue to remaining particles
				continue;
			}
			++MotionlessCount;

			// Particle sleep counter needs to be updated for particles to be able to sleep if we switch to partial island sleep
			const int32 SleepCounter = FMath::Min(int32(Rigid->SleepCounter()) + 1, Node->SleepCounterThreshold);
			Rigid->SetSleepCounter(int8(FMath::Min(SleepCounter, TNumericLimits<int8>::Max())));

			// Take the longest sleep time
			SleepCounterThreshold = FMath::Max(SleepCounterThreshold, Node->SleepCounterThreshold);
		}

		int32 SleepCounter = 0;
		if (bWithinSleepThreshold)
		{
			// If we get here we may want to sleep
			SleepCounter = Island->SleepCounter + 1;
			if (SleepCounter > SleepCounterThreshold)
			{
				Island->Flags.bIsSleeping = true;
				SleepCounter = 0;
			}
		}
		Island->SleepCounter = SleepCounter;

		SwitchIslandSleepType(Island, MotionlessCount);
	}

	void FPBDIslandManager::PropagateIslandSleep(FPBDIsland* Island)
	{
		PropagateIslandSleepToParticles(Island);
		PropagateIslandSleepToConstraints(Island);
	}

	void FPBDIslandManager::PropagateIslandSleepToParticles(FPBDIsland* Island)
	{
		bool bRebuildViews = false;
		for (FPBDIslandParticle* IslandNode : Island->Nodes)
		{
			if (Island->Flags.bIsSleeping != IslandNode->Flags.bIsSleeping)
			{
				FGeometryParticleHandle* Particle = IslandNode->GetParticle();

				// Put to sleep...
				if (Island->Flags.bIsSleeping)
				{
					Particles.DeactivateParticle(Particle, true);
				}

				// Wake up...
				if (!Island->Flags.bIsSleeping)
				{
					Particles.ActivateParticle(Particle, true);

					// When we wake particles, we have skipped their integrate step which causes some issues:
					//	- we have zero velocity (no gravity or external forces applied)
					//	- the world transforms cached in the ShapesArray will be at the last post-integrate positions
					//	  which doesn't match what the velocity is telling us
					// This causes problems for the solver - essentially we have an "initial overlap" situation.
					// @todo(chaos): We could just run (partial) integrate here for this particle, but we don't know about the Evolution - fix this
					// Better solution: Leave Particle X and P in current state for sleeping particles (rather than setting X = P)
					for (const TUniquePtr<FPerShapeData>& Shape : Particle->ShapesArray())
					{
						Shape->UpdateLeafWorldTransform(Particle);
					}
				}

				IslandNode->Flags.bIsSleeping = Island->Flags.bIsSleeping;
				bRebuildViews = true;
			}
		}

		// Update views if we changed particle state
		if (bRebuildViews)
		{
			Particles.RebuildViews();
		}
	}

	void FPBDIslandManager::PropagateIslandSleepToConstraints(FPBDIsland* Island)
	{
		// Set the constraint sleep state to match
		for (TArray<FPBDIslandConstraint*>& IslandEdges : Island->ContainerEdges)
		{
			for (FPBDIslandConstraint* IslandEdge : IslandEdges)
			{
				if (Island->Flags.bIsSleeping != IslandEdge->Flags.bIsSleeping)
				{
					IslandEdge->GetConstraint()->SetIsSleeping(Island->Flags.bIsSleeping);
					IslandEdge->Flags.bIsSleeping = Island->Flags.bIsSleeping;
				}
			}
		}
	}

	void FPBDIslandManager::ProcessPartialIslandSleep(FPBDIsland* Island, const FRealSingle Dt)
	{
		// Make all non-moving particles in the island go to sleep
		SleepNonMovingParticles(Island, Dt);

		bool bHaveParticlesBeenWokenUp = false;
		if (CVars::ChaosSolverPartialSleepCollisionConstraintsOnly >= 1)
		{
			// If this island contains any constraint which don't support partial sleeping,
			// we need to traverse the constraints to make sure that connected particles are either both sleeping or both awake.
			bHaveParticlesBeenWokenUp = RewakeConstraintsNotSupportingPartialSleep(Island);
		}

		if (bHaveParticlesBeenWokenUp)
		{
			// After re-waking, we need to sync all constraints to the particle state.
			// Note that it is not sufficient to call @SleepConstraintsBetweenNonAwakeParticles because more particles might be awake than before the solve.
			SyncConstraintSleepStateToParticles(Island);
		}
		else
		{
			// Without re-waking:
			// Deactivate all adjacent constraints between non-awake particles, i.e. (a) 2 sleeping particles or (b) a sleeping and a non-dynamic particle.
			// Processing non-sleeping constraints is sufficient because no particles were woken up since the last call to @SyncConstraintSleepStateToParticles
			SleepConstraintsBetweenNonAwakeParticles(Island);
		}

		// NOTE: If (ChaosSolverPostStepWakeThreshold > 0.0f), we might encounter situations where an active-sleeping pair is connected by a non-supporting constraint. 
		// However, we tolerate this since the state should only last for one tick before RewakeConstraintsNotSupportingPartialSleep will sync again.
		if (CVars::ChaosSolverPostStepWakeThreshold > 0.0f)
		{
			// Process all constraints connecting active-sleeping pairs.
			// Wake sleeping body if the constraint solver applies a significant impulse.
			// Only one layer of sleeping particles can be woken up per tick (i.e. no propagation through layers).
			PostStepPartialWake(Island);
		}

		// Debugging only: Validate that the constraint sleep state at the end of the step matches the particle sleep states
		if (CVars::bChaosSolverValidateConstraintSleepState)
		{
			ValidateConstraintSleepState(Island);
		}
	}

	void FPBDIslandManager::SleepNonMovingParticles(FPBDIsland* Island, const FRealSingle Dt)
	{
		bool bAllNodesSleeping = true;
		int32 MotionlessCount = 0;

		TArray<FGeometryParticleHandle*> SleptParticles;
		for (FPBDIslandParticle* Node : Island->GetParticles())
		{
			// All zeroes means never sleep/disable
			if ((Node->SleepLinearThresholdSq <= 0) && (Node->SleepAngularThresholdSq <= 0))
			{
				bAllNodesSleeping = false;
				continue;
			}

			// No need to check sleeping nodes again
			if (!!Node->Flags.bIsSleeping)
			{
				++MotionlessCount;
				continue;
			}

			FPBDRigidParticleHandle* Rigid = Node->GetParticle()->CastToRigidParticle();
			if (Rigid == nullptr)
			{
				continue;
			}

			// The particle SleepType may have changed since the node cached thresholds
			// This is needed because we have API on the particle to SetSleepType which doesn't end up updating thresholds which is done when setting SleepType via FPBDRigidsEvolutionBase::SetParticleSleepType
			if (Rigid->SleepType() == ESleepType::NeverSleep)
			{
				bAllNodesSleeping = false;
				continue;
			}

			// Check the particle state against the thresholds
			int32 SleepCounterThreshold = Node->SleepCounterThreshold;
			int32 SleepCounter = 0;
			if (SleepCounterThreshold < TNumericLimits<int32>::Max())
			{
				// Particles have a max sleep counter of 127 (to reduce counter space in the particle)
				SleepCounterThreshold = FMath::Min(SleepCounterThreshold, TNumericLimits<int8>::Max());

				UpdateParticleSleepMetrics(*Rigid, Dt);

				// Did we exceed the velocity threshold?
				if ((Rigid->VSmooth().SizeSquared() > Node->SleepLinearThresholdSq)
					|| (Rigid->WSmooth().SizeSquared() > Node->SleepAngularThresholdSq))
				{
					bAllNodesSleeping = false;
				}
				else // motionless
				{
					++MotionlessCount;

					// If we get here we might want to sleep
					// Update the sleep counter and sleep if we exceed it
					static_assert(sizeof(decltype(Rigid->SleepCounter())) == 1, "Expected int8 for SleepCounter(). Update clamp below");
					SleepCounter = FMath::Min(int32(Rigid->SleepCounter()) + 1, int32(TNumericLimits<int8>::Max()));
					if (SleepCounter > SleepCounterThreshold)
					{
						SleptParticles.Add(Rigid->Handle());
						Node->Flags.bIsSleeping = true;
						SleepCounter = 0;
					}
					else
					{
						bAllNodesSleeping = false;
					}
				}
			}
			Rigid->SetSleepCounter(int8(SleepCounter));
		}

		// Flag the island if all particles are sleeping
		if (bAllNodesSleeping)
		{
			Island->Flags.bIsSleeping = true;
			Island->SleepCounter = 0;
		}
		else if (MotionlessCount < Island->GetNumParticles())
		{
			// Reset island sleep counter if there are any moving particles
			Island->SleepCounter = 0;
		}

		if (!SleptParticles.IsEmpty())
		{
			Particles.DeactivateParticles(SleptParticles);
		}

		SwitchIslandSleepType(Island, MotionlessCount);
	}

	bool FPBDIslandManager::RewakeConstraintsNotSupportingPartialSleep(FPBDIsland* Island)
	{
		// NOTE: We use node flags here since we want the updated state when processing.
		const auto IsSleepingAndDynamic = [](const FPBDIslandParticle* Node) -> bool
			{
				return Node && !!Node->Flags.bIsSleeping && !!Node->Flags.bIsDynamic;
			};

		auto AddToWakeQueue = [](FPBDIslandParticle* Node, TArray<FGeometryParticleHandle*>& WakeUpQueue, TArray<FPBDIslandParticle*>& NodeQueue)
			{
				Node->Flags.bIsSleeping = false;
				NodeQueue.Add(Node);
				if (FGeometryParticleHandle* Particle = Node->GetParticle())
				{
					WakeUpQueue.Add(Particle);

					// If the particle was re-woken, we need to restore the sleep counter from before the reset to 0 in SleepNonMovingParticles.
					// We know that the particle could only go to sleep if Rigid->SleepCounter() > Node->SleepCounterThreshold.
					// Thus, it holds that Rigid->SleepCounter() == Node->SleepCounterThreshold before the reset. 
					// If the particle was not just deactivated, it would be woken up to prevent partial sleeping for non-supporting constraints and be set to the same counter.
					if (FPBDRigidParticleHandle* Rigid = Particle->CastToRigidParticle())
					{
						Rigid->SetSleepCounter(static_cast<int8>(FMath::Min(Node->SleepCounterThreshold, TNumericLimits<int8>::Max())));
					}
				}
			};

		TArray<FGeometryParticleHandle*> WakeUpQueue;
		TArray<FPBDIslandParticle*> NodeQueue;
		WakeUpQueue.Reserve(Island->GetNumParticles());
		NodeQueue.Reserve(Island->GetNumParticles());

		// Check all awake constraints to make sure there are no active-sleeping particle pairs
		// connected by constraints which do not support partial island sleep.
		// Add all particles that need to re-woken to NodeQueue.
		for (TArray<FPBDIslandConstraint*>& IslandEdges : Island->ContainerEdges)
		{
			for (FPBDIslandConstraint* IslandEdge : IslandEdges)
			{
				// Skip already sleeping constraints since must connect particles which were already 
				// sleeping or kinematic during the solve and can therefore not have been just put to sleep.
				if (!IslandEdge->Flags.bIsSleeping)
				{
					FPBDIslandParticle* Node0 = IslandEdge->Nodes[0];
					FPBDIslandParticle* Node1 = IslandEdge->Nodes[1];
					const FConstraintHandle* Constraint = IslandEdge->GetConstraint();

					if (Node0 != nullptr && Node1 != nullptr)
					{
						// Reactivate particles between constraints which do not support partial sleeping
						if (Constraint && !Constraint->SupportsPartialIslandSleeping())
						{
							// Cases for re-waking
							// a) One particle asleep, one particle awake, both dynamic.
							const bool bIsSleeping0 = IsSleepingAndDynamic(Node0);
							const bool bIsSleeping1 = IsSleepingAndDynamic(Node1);
							const bool bIsAwake0 = Node0->Flags.bIsDynamic && !bIsSleeping0;
							const bool bIsAwake1 = Node1->Flags.bIsDynamic && !bIsSleeping1;
							if (bIsSleeping0 && bIsAwake1)
							{
								IslandEdge->Flags.bWasProcessed = true;
								AddToWakeQueue(Node0, WakeUpQueue, NodeQueue);
							}
							else if (bIsAwake0 && bIsSleeping1)
							{
								IslandEdge->Flags.bWasProcessed = true;
								AddToWakeQueue(Node1, WakeUpQueue, NodeQueue);
							}
						}
					}
				}
			}
		}

		// Walk the constraint graph to re-wake all particles connected to particles in NodeQueue
		// by constraints which do not support partial sleeping.
		// This ensures that all particles connected by non-supporting constraints have the same sleep/awake state.
		for (int32 NodeIndex = 0; NodeIndex < NodeQueue.Num(); ++NodeIndex)
		{
			FPBDIslandParticle* ThisNode = NodeQueue[NodeIndex];
			for (FPBDIslandConstraint* Edge : ThisNode->Edges)
			{
				if (!Edge->Flags.bWasProcessed)
				{
					Edge->Flags.bWasProcessed = true;

					const FConstraintHandle* Constraint = Edge->GetConstraint();
					if (Constraint && !Constraint->SupportsPartialIslandSleeping())
					{
						FPBDIslandParticle* OtherNode = (ThisNode == Edge->Nodes[0]) ? Edge->Nodes[1] : Edge->Nodes[0];
						if ((OtherNode != nullptr) && OtherNode->Flags.bIsDynamic)
						{
							if (IsSleepingAndDynamic(OtherNode))
							{
								AddToWakeQueue(OtherNode, WakeUpQueue, NodeQueue);
							}
						}
					}
				}
			}
		}

		// Don't reset the sleep counter here since they have already been set in AddToWakeQueue.
		WakeUpParticles(WakeUpQueue, /*bResetSleepCounter=*/ false);
		ResetEdgeFlags(Island);

		return !WakeUpQueue.IsEmpty();
	}

	void FPBDIslandManager::SleepConstraintsBetweenNonAwakeParticles(FPBDIsland* Island)
	{
		for (TArray<FPBDIslandConstraint*>& IslandEdges : Island->ContainerEdges)
		{
			for (FPBDIslandConstraint* IslandEdge : IslandEdges)
			{
				// Skip already sleeping constraints
				if (!IslandEdge->Flags.bIsSleeping)
				{
					const FPBDIslandParticle* Node0 = IslandEdge->Nodes[0];
					const FPBDIslandParticle* Node1 = IslandEdge->Nodes[1];
					FConstraintHandle* Constraint = IslandEdge->GetConstraint();

					if (Node0 != nullptr && Node1 != nullptr)
					{
						// Deactivate constraint if:
						// Case (a) Both particles are sleeping
						// Case (b) One particle is sleeping and the other is not dynamic (i.e. kinematic, static, etc.)
						const bool bAreBothSleeping = !!Node0->Flags.bIsSleeping && !!Node1->Flags.bIsSleeping;
						const bool bIsSleeping0Kinematic1 = !!Node0->Flags.bIsSleeping && !Node1->Flags.bIsDynamic;
						const bool bIsSleeping1Kinematic0 = !!Node1->Flags.bIsSleeping && !Node0->Flags.bIsDynamic;
						if (bAreBothSleeping || bIsSleeping0Kinematic1 || bIsSleeping1Kinematic0)
						{
							IslandEdge->Flags.bIsSleeping = true;
							Constraint->SetIsSleeping(true);
						}
					}
					else
					{
						// If one node is NULL, the edge sleep state should match the other node.
						if (Node0 != nullptr) // && Node1 == nullptr
						{
							IslandEdge->Flags.bIsSleeping = Node0->Flags.bIsSleeping;
							Constraint->SetIsSleeping(Node0->Flags.bIsSleeping);
						}
						else if (Node1 != nullptr) // && Node0 == nullptr
						{
							IslandEdge->Flags.bIsSleeping = Node1->Flags.bIsSleeping;
							Constraint->SetIsSleeping(Node1->Flags.bIsSleeping);
						}
						else
						{
							IslandEdge->Flags.bIsSleeping = false;
							Constraint->SetIsSleeping(false);
						}
					}
				}
			}
		}
	}

	void FPBDIslandManager::PostStepPartialWake(FPBDIsland* Island)
	{
		const auto IsSleeping = [](const FPBDIslandParticle* Node) -> bool
			{
				if (Node == nullptr)
				{
					return false;
				}
				const FPBDRigidParticleHandle* Rigid = Node->GetParticle()->CastToRigidParticle();
				if (Rigid == nullptr)
				{
					return false;
				}

				// NOTE: Make sure to use the particle flag here in order not to consider particles
				// newly woken up by this method when checking active-sleeping pairs
				return Rigid->IsSleeping();
			};

		const auto CheckWakeActiveSleepingPair = [](FPBDIslandParticle* Node, FPBDIslandConstraint* IslandEdge, TArray<FGeometryParticleHandle*>& WakeUpQueue)
			{
				if (FPBDRigidParticleHandle* Rigid = Node->GetParticle()->CastToRigidParticle())
				{
					FConstraintHandle* Constraint = IslandEdge->GetConstraint();

					// NetImpulse = Mass * change in velocity
					const FVec3f VelocitySolverImpulse = Constraint->GetNetLinearImpulse();
					const FVec3f DiffVelocity = VelocitySolverImpulse / (FRealSingle)Rigid->M();

					if (DiffVelocity.SizeSquared() > FMath::Square(CVars::ChaosSolverPostStepWakeThreshold))
					{
						Node->Flags.bIsSleeping = false;
						WakeUpQueue.Add(Node->GetParticle());
						// Set the sleep counter so that the particle can sleep again after 1 step.
						Rigid->SetSleepCounter(int8(Node->SleepCounterThreshold));

						// If a particle is active, we know for sure that all connected constraints need to be active too.
						// Only constraints between sleeping particles and kinematics can sleep.
						for (FPBDIslandConstraint* Edge : Node->Edges)
						{
							if (Edge->Flags.bIsSleeping)
							{
								Edge->Flags.bIsSleeping = false;
								if (FConstraintHandle* ConnectedContraint = Edge->GetConstraint())
								{
									ConnectedContraint->SetIsSleeping(false);
								}
							}
						}
					}
				}
			};

		TArray<FGeometryParticleHandle*> WakeUpQueue;
		for (TArray<FPBDIslandConstraint*>& IslandEdges : Island->ContainerEdges)
		{
			for (FPBDIslandConstraint* IslandEdge : IslandEdges)
			{
				// Skip already sleeping constraints
				if (!IslandEdge->Flags.bIsSleeping)
				{
					FPBDIslandParticle* Node0 = IslandEdge->Nodes[0];
					FPBDIslandParticle* Node1 = IslandEdge->Nodes[1];
					FConstraintHandle* Constraint = IslandEdge->GetConstraint();

					if (Node0 != nullptr && Node1 != nullptr)
					{
						const bool bIsSleeping0 = IsSleeping(Node0);
						const bool bIsSleeping1 = IsSleeping(Node1);
						const bool bIsAwake0 = Node0->Flags.bIsDynamic && !bIsSleeping0;
						const bool bIsAwake1 = Node1->Flags.bIsDynamic && !bIsSleeping1;

						// Reactivate the body if:
						// (a) One particle is sleeping and the other is awake AND
						// (b) The constraint between them has a significant net impulse.
						if (bIsAwake0 && bIsSleeping1)
						{
							CheckWakeActiveSleepingPair(Node1, IslandEdge, WakeUpQueue);
						}
						else if (bIsAwake1 && bIsSleeping0)
						{
							CheckWakeActiveSleepingPair(Node0, IslandEdge, WakeUpQueue);
						}
					}
				}
			}
		}
		WakeUpParticles(WakeUpQueue, /*bResetSleepCounter=*/false);
	}

	void FPBDIslandManager::SwitchIslandSleepType(FPBDIsland* Island, const int32 MotionlessCount)
	{
		if (Island->GetNumParticles() > 0)
		{
			const FReal RatioMotionless = MotionlessCount / (FReal)Island->GetNumParticles();

			const bool bIsBelowMinMotionlessRatio = RatioMotionless < CVars::ChaosSolverMinMotionlessRatio;
			const bool bExceedSleepCounterSwitchThreshold = Island->SleepTypeSwitchCounter >= CVars::ChaosSolverSleepModeSwitchThreshold;

			const bool bPotentialSwitchToPartialIslandSleep = !Island->Flags.bEnoughMotionlessNodesForPartialSleep && !bIsBelowMinMotionlessRatio;
			const bool bPotentialSwitchToFullIslandSleep = !!Island->Flags.bEnoughMotionlessNodesForPartialSleep && bIsBelowMinMotionlessRatio;

			// Switch sleep type: flip the switch and reset the counter
			const bool bSwitchSleepType = (bPotentialSwitchToPartialIslandSleep || bPotentialSwitchToFullIslandSleep) && bExceedSleepCounterSwitchThreshold;

			// Increase counter: potential switch but not enough ticks
			const bool bIncreaseCounter = (bPotentialSwitchToPartialIslandSleep || bPotentialSwitchToFullIslandSleep) && !bExceedSleepCounterSwitchThreshold;

			if (bSwitchSleepType)
			{
				Island->Flags.bEnoughMotionlessNodesForPartialSleep = !Island->Flags.bEnoughMotionlessNodesForPartialSleep ? true : false;
				Island->SleepTypeSwitchCounter = 0;
			}
			else if (bIncreaseCounter)
			{
				++(Island->SleepTypeSwitchCounter);
			}
			else // Reset counter
			{
				Island->SleepTypeSwitchCounter = 0;
			}
		}
	}

	void FPBDIslandManager::ValidateConstraintSleepState(const FPBDIsland* Island)
	{
		for (const TArray<FPBDIslandConstraint*>& IslandEdges : Island->ContainerEdges)
		{
			for (const FPBDIslandConstraint* IslandEdge : IslandEdges)
			{
				const FPBDIslandParticle* Node0 = IslandEdge->Nodes[0];
				const FPBDIslandParticle* Node1 = IslandEdge->Nodes[1];
				FConstraintHandle* Constraint = IslandEdge->GetConstraint();

				if (Node0 != nullptr && Node1 != nullptr)
				{
					const bool bIsSleeping0 = IsParticleSleeping(Node0->GetParticle());
					const bool bIsSleeping1 = IsParticleSleeping(Node1->GetParticle());
					const bool bIsDynamic0 = IsParticleDynamic(Node0->GetParticle());
					const bool bIsDynamic1 = IsParticleDynamic(Node1->GetParticle());

					// Constraint should sleep if
					// Case (a) Both particles are sleeping
					// Case (b) One particle is sleeping and the other is not dynamic (i.e. kinematic, static, etc.)
					const bool bAreBothSleeping = bIsSleeping0 && bIsSleeping1;
					const bool bIsSleeping0Kinematic1 = bIsSleeping0 && !bIsDynamic1;
					const bool bIsSleeping1Kinematic0 = bIsSleeping1 && !bIsDynamic0;
					if (bAreBothSleeping || bIsSleeping0Kinematic1 || bIsSleeping1Kinematic0)
					{
						check(!!IslandEdge->Flags.bIsSleeping);
						check(!Constraint->SupportsSleeping() || Constraint->IsSleeping());
					}

					// Constraint should not sleep if
					// Case (a) at least one particle is awake
					const bool bIsAwakeDynamic0 = !bIsSleeping0 && bIsDynamic0;
					const bool bIsAwakeDynamic1 = !bIsSleeping1 && bIsDynamic1;
					if (bIsAwakeDynamic0 || bIsAwakeDynamic1)
					{
						check(!IslandEdge->Flags.bIsSleeping);
						check(!Constraint->SupportsSleeping() || !Constraint->IsSleeping());
					}
				}
				else
				{
					// If one node is NULL, the edge sleep state should match the other node.
					if (Node0 != nullptr) // && Node1 == nullptr
					{
						check(IslandEdge->Flags.bIsSleeping == Node0->Flags.bIsSleeping);
						check(!Constraint->SupportsSleeping() || Constraint->IsSleeping() == Node0->Flags.bIsSleeping);
					}
					else if (Node1 != nullptr) // && Node0 == nullptr
					{
						check(IslandEdge->Flags.bIsSleeping == Node1->Flags.bIsSleeping);
						check(!Constraint->SupportsSleeping() || Constraint->IsSleeping() == Node1->Flags.bIsSleeping);
					}
					else // Node0 == nullptr && Node1 == nullptr
					{
						check(!IslandEdge->Flags.bIsSleeping);
						check(!Constraint->SupportsSleeping() || !Constraint->IsSleeping());
					}
				}
			}
		}
	}

	// If (bProcessEdgeBasedWake), we first process all updated nodes and triggered wake events, then we process all new edges and wake events.
	// If (!bProcessEdgeBasedWake), we only process updated nodes and their wake events.
	void FPBDIslandManager::ProcessPartialIslandWake(FPBDIsland* Island, const bool bProcessEdgeBasedWake)
	{
		TArray<FGeometryParticleHandle*> ParticlesToWakeUp; // Particles which will be woken up after processing all partial wake events.
		TArray<FPBDIslandParticle*> NodeQueue;				// Queue of island nodes to be traversed by partial waking.
		TArray<FPBDParticleMomentumSq> MomentumQueue;		// Linear and angular momentum of island nodes in NodeQueue.
		NodeQueue.Reserve(Island->Nodes.Num());
		if (CVars::ChaosSolverMomentumPropagation > 0.0f)
		{
			MomentumQueue.Reserve(Island->Nodes.Num());
		}

		// Phase 1 - Waking triggered by particle state changes
		// Step 1a - Find trigger particles (i.e. particles which were just woken up the user) 
		// Start with node-based waking since it always triggers independent of the particle velocity.
		for (FPBDIslandParticle* Node : Island->Nodes)
		{
			// Partial wake events need to be performed for bAreSleepThresholdsNonPositive and bIsParticleNeverSleep.
			// Otherwise, we might end up with sleeping particles above a never-sleeping or zero-sleep-threshold particle.
			const bool bAreSleepThresholdsNonPositive = (Node->SleepLinearThresholdSq <= 0) && (Node->SleepAngularThresholdSq <= 0);
			bool bIsParticleNeverSleep = false;
			if (FPBDRigidParticleHandle* Rigid = Node->Particle->CastToRigidParticle())
			{
				bIsParticleNeverSleep = Rigid->SleepType() == ESleepType::NeverSleep;
			}

			if ((Node->Flags.bWasUpdated || bAreSleepThresholdsNonPositive || bIsParticleNeverSleep) && !Node->Flags.bWasProcessed)
			{
				ProcessTriggerParticle(Node, NodeQueue, MomentumQueue, /*bAlwaysTrigger=*/ true);
				Node->Flags.bWasProcessed = true;

				// Reset this this to pre-integration velocity for nodes whose transforms were set explicitly.
				// We previously overwrote VSmooth with the penetration velocity (i.e. velocity to cause penetration after explicitly setting the transform).
				// We also check (bProcessEdgeBasedWake) to make sure we call this function from FinalizeIslands not from UpdateExplicitSleep.
				if (Node->Flags.bWasTransformSetExplicitly && bProcessEdgeBasedWake)
				{
					if (FPBDRigidParticleHandle* Rigid = Node->Particle->CastToRigidParticle())
					{
						Rigid->SetVSmooth(Rigid->GetPreV());
					}
				}
			}
		}
		const int32 NumNodesNodeWake = NodeQueue.Num();
		ValidateWakeQueues(NodeQueue, MomentumQueue);
		bool bResetEdgeFlags = false;

		// Step 1b - Propagate wake event
		// Partial island wake disabled, but at least one trigger particle found? Then, perform a full island wake.
		if (!CVars::bChaosSolverPartialIslandWake && NumNodesNodeWake > 0)
		{
			PropagateIslandSleep(Island);

			// Particle sleep counter reset is needed to prevent immediate re-sleeping in partial island sleeping.
			ResetParticleSleepCounter(Island);
		}
		else // Otherwise, propagate the partial wake event through the constraint graph.
		{
			bResetEdgeFlags = PropagatePartialIslandWake(NodeQueue, MomentumQueue, ParticlesToWakeUp, NumNodesNodeWake, /*bTriggerNodesAlwaysPropagate=*/ true);
		}

		NodeQueue.Reset();
		MomentumQueue.Reset();

		if (bProcessEdgeBasedWake)
		{
			// Phase 2 - Waking triggered by new constraints
			// Step 2a - Find trigger particles (i.e. particles connected to new constraints with significant motion)
			// Edge-based waking will only be triggered for nodes that haven't been processed in node-based waking yet.
			for (TArray<FPBDIslandConstraint*>& IslandEdges : Island->ContainerEdges)
			{
				for (FPBDIslandConstraint* IslandEdge : IslandEdges)
				{
					if (IslandEdge->Flags.bIsNew)
					{
						FPBDIslandParticle* Node0 = IslandEdge->Nodes[0];
						FPBDIslandParticle* Node1 = IslandEdge->Nodes[1];
						if (Node0 && !Node0->Flags.bWasProcessed)
						{
							ProcessTriggerParticle(Node0, NodeQueue, MomentumQueue, /*bAlwaysTrigger=*/ !CVars::bChaosSolverWakeOnMovingOnly);
							Node0->Flags.bWasProcessed = true;
						}
						if (Node1 && !Node1->Flags.bWasProcessed)
						{
							ProcessTriggerParticle(Node1, NodeQueue, MomentumQueue, /*bAlwaysTrigger=*/ !CVars::bChaosSolverWakeOnMovingOnly);
							Node1->Flags.bWasProcessed = true;
						}
					}
				}
			}
			const int32 NumNodesConstraintWake = NodeQueue.Num();
			ValidateWakeQueues(NodeQueue, MomentumQueue);

			// Step 2b - Propagate wake event
			// Partial island wake disabled, but at least one trigger particle found? Then, perform a full island wake.
			if (!CVars::bChaosSolverPartialIslandWake && NumNodesConstraintWake > 0)
			{
				PropagateIslandSleep(Island);

				// Particle sleep counter reset is needed to prevent immediate re-sleeping in partial island sleeping.
				ResetParticleSleepCounter(Island);
			}
			else // Otherwise, propagate the partial wake event through the constraint graph.
			{
				// Reset edge flags if we previously processed partial wake events.
				if (bResetEdgeFlags && !NodeQueue.IsEmpty())
				{
					ResetEdgeFlags(Island);
				}
				PropagatePartialIslandWake(NodeQueue, MomentumQueue, ParticlesToWakeUp, NumNodesConstraintWake, /*bTriggerNodesAlwaysPropagate=*/ false);
			}
		}

		// Phase 3 - Wake up all relevant particles and reset sleep counter
		WakeUpParticles(ParticlesToWakeUp, /*bResetSleepCounter=*/ false);
	}

	bool FPBDIslandManager::PropagatePartialIslandWake(TArray<FPBDIslandParticle*>& NodeQueue, TArray<FPBDParticleMomentumSq>& MomentumQueue,
		TArray<FGeometryParticleHandle*>& ParticlesToWakeUp, const int32 NumNodesInQueue, const bool bTriggerNodesAlwaysPropagate)
	{
		bool bResetEdgeFlags = false;
		for (int32 NodeIndex = 0; NodeIndex < NodeQueue.Num(); ++NodeIndex)
		{
			bResetEdgeFlags = true;
			FPBDIslandParticle* Node = NodeQueue[NodeIndex];

			// Traverse all edges connected to a node in the wake-up queue.
			for (FPBDIslandConstraint* Edge : Node->Edges)
			{
				if (!Edge->Flags.bWasProcessed)
				{
					Edge->Flags.bWasProcessed = true;

					FPBDIslandParticle* OtherNode = (Node == Edge->Nodes[0]) ? Edge->Nodes[1] : Edge->Nodes[0];
					if ((OtherNode != nullptr) && OtherNode->Flags.bIsDynamic)
					{
						// Wake and propagate always to the next constraint if
						const bool bIsTriggerNode = NodeIndex < NumNodesInQueue;
						const bool bIsConfirmedPropagation = bIsTriggerNode && (Edge->Flags.bIsNew || bTriggerNodesAlwaysPropagate);
						const bool bIsUpwardPropagation = ShouldAssignLevels(Edge->Island) ? OtherNode->Level >= Edge->Level : false;	// Propagation upward through the levels, i.e. away from closest kinematic
						const bool bEdgePropagatesPartialWake = (CVars::ChaosSolverPartialSleepCollisionConstraintsOnly != 0) && Edge->GetConstraint() && !Edge->GetConstraint()->SupportsPartialIslandSleeping();

						// 1) We process one of the trigger nodes and propagate through a new constraint (if bTriggerNodesAlwaysPropagate == false).
						// 2) We propagate upward in the constraint graph levels (away from kinematic).
						// 3) Partial sleep is enabled for collisions only and the constraint type does not support connecting sleeping and active particles.
						if (bIsConfirmedPropagation || bIsUpwardPropagation || bEdgePropagatesPartialWake)
						{
							const FPBDParticleMomentumSq* NodeMomentum = MomentumQueue.IsEmpty() ? nullptr : &MomentumQueue[NodeIndex];
							ProcessParticleWake(OtherNode, NodeQueue[NodeIndex], NodeMomentum, NodeQueue, MomentumQueue, ParticlesToWakeUp, /*bAlwaysPropagate=*/ true);
						}
						// 3) We propagate the impact momentum of the trigger particle through the constraint graph.
						else if (CVars::ChaosSolverMomentumPropagation > 0.0f)
						{
							check(!MomentumQueue.IsEmpty());
							ProcessParticleWake(OtherNode, NodeQueue[NodeIndex], &MomentumQueue[NodeIndex], NodeQueue, MomentumQueue, ParticlesToWakeUp, /*bAlwaysPropagate=*/ false);
						}

						ValidateWakeQueues(NodeQueue, MomentumQueue);
					}
				}
			}
		}
		return bResetEdgeFlags;
	}

	bool FPBDIslandManager::ProcessTriggerParticle(FPBDIslandParticle* Node, TArray<FPBDIslandParticle*>& NodeQueue, TArray<FPBDParticleMomentumSq>& MomentumQueue, const bool bAlwaysTrigger)
	{
		const auto IsMoving = [](FPBDIslandParticle* Node) -> bool
			{
				if (Node == nullptr)
				{
					return false;
				}
				FPBDRigidParticleHandle* Rigid = Node->GetParticle()->CastToRigidParticle();
				if (Rigid == nullptr)
				{
					return false;
				}

				// Use the smoothed velocity to check for motion, unless it is zero (i.e. was never set).
				FVec3f V = Rigid->VSmooth();
				FVec3f W = Rigid->WSmooth();
				if (V.SizeSquared() == 0 && W.SizeSquared() == 0)
				{
					// Use the regular velocity from the last time step
					V = Rigid->GetPreV();
					W = Rigid->GetPreW();
				}

				if ((V.SizeSquared() > Node->WakeLinearThresholdSq)
					|| (W.SizeSquared() > Node->WakeAngularThresholdSq))
				{
					return true;
				}

				return false;
			};

		const auto IsAwake = [](const FPBDIslandParticle* Node) -> bool
			{
				return Node && !IsParticleSleeping(Node->GetParticle()) && IsParticleDynamic(Node->GetParticle());
			};

		const auto IsDynamic = [](const FPBDIslandParticle* Node) -> bool
			{
				return Node && IsParticleDynamic(Node->GetParticle());
			};

		const auto ComputeLinearMomentumSq = [](const FPBDIslandParticle* Node) -> FRealSingle
			{
				if (Node == nullptr)
				{
					return FRealSingle(0);
				}
				const FPBDRigidParticleHandle* Rigid = Node->GetParticle()->CastToRigidParticle();
				if (Rigid == nullptr)
				{
					return FRealSingle(0);
				}

				// Use the smoothed velocity to check for motion, unless it is zero (i.e. was never set).
				FVec3f V = Rigid->VSmooth();
				if (V.SizeSquared() == 0)
				{
					// Use the regular velocity from the last time step
					V = Rigid->GetPreV();
				}
				const FRealSingle Mass = FRealSingle(Rigid->M());
				const FVec3f LinearMomentum = Mass * V;
				return LinearMomentum.SizeSquared();
			};

		const auto ComputeAngularMomentumSq = [](const FPBDIslandParticle* Node)->FRealSingle
			{
				if (Node == nullptr)
				{
					return FRealSingle(0);
				}
				const FPBDRigidParticleHandle* Rigid = Node->GetParticle()->CastToRigidParticle();
				if (Rigid == nullptr)
				{
					return FRealSingle(0);
				}

				// Use the smoothed velocity to check for motion, unless it is zero (i.e. was never set).
				FVec3f W = Rigid->WSmooth();
				if (W.SizeSquared() == 0)
				{
					// Use the regular velocity from the last time step
					W = Rigid->GetPreW();
				}

				// Compute the magnitude of the angular momentum in body-space (equivalent and more efficient)
				const FMatrix33 Transform = (Rigid->GetR() * Rigid->RotationOfMass()).ToMatrix().GetTransposed();
				const FVec3f WLocal = Transform * W;
				const FVec3f AngularMomentum = Rigid->I() * WLocal;
				return AngularMomentum.SizeSquared();
			};

		// Stop wake-up event if the node or particle is empty
		if (Node == nullptr)
		{
			return false;
		}
		FPBDRigidParticleHandle* Rigid = Node->GetParticle()->CastToRigidParticle();
		if (Rigid == nullptr)
		{
			return false;
		}

		const bool bIsAwake = IsAwake(Node);
		const bool bIsDynamic = IsDynamic(Node);

		// Stop wake-up event if the particle is not dynamic
		if (!bIsDynamic)
		{
			return false;
		}

		// The particle must be awake to trigger a partial wake-up event.
		if (bIsAwake)
		{
			// Add node to the queue if (a) above the motion wake threshold or (b) always triggering.
			if (bAlwaysTrigger || IsMoving(Node))
			{
				if (CVars::ChaosSolverMomentumPropagation > 0.0f)
				{
					const FRealSingle LinearMomentumSq = ComputeLinearMomentumSq(Node);
					const FRealSingle AngularMomentumSq = ComputeAngularMomentumSq(Node);
					MomentumQueue.Add(FPBDParticleMomentumSq(LinearMomentumSq, AngularMomentumSq));
				}
				NodeQueue.Add(Node);
				return true;
			}
		}

		return false;
	}

	bool FPBDIslandManager::ProcessParticleWake(FPBDIslandParticle* Node, const FPBDIslandParticle* OtherNode, const FPBDParticleMomentumSq* OtherMomentumSq,
		TArray<FPBDIslandParticle*>& NodeQueue, TArray<FPBDParticleMomentumSq>& MomentumQueue, TArray<FGeometryParticleHandle*>& WakeUpQueue,
		const bool bAlwaysPropagate)
	{
		const auto IsAwake = [](const FPBDIslandParticle* Node) -> bool
			{
				return Node && !IsParticleSleeping(Node->GetParticle()) && IsParticleDynamic(Node->GetParticle());
			};

		const auto IsDynamic = [](const FPBDIslandParticle* Node) -> bool
			{
				return Node && IsParticleDynamic(Node->GetParticle());
			};

		// Stop wake-up event if the node or particle is empty
		if (Node == nullptr)
		{
			return false;
		}
		FPBDRigidParticleHandle* Rigid = Node->GetParticle()->CastToRigidParticle();
		if (Rigid == nullptr)
		{
			return false;
		}

		const bool bIsAwake = IsAwake(Node);
		const bool bIsDynamic = IsDynamic(Node);
		const int32 SleepCounterValue = 0;

		// Stop wake-up event if the particle is not dynamic
		if (!bIsDynamic)
		{
			return false;
		}

		// No need for momentum computation, if momentum propagation disabled and we always propagate to the next particle.
		if (CVars::ChaosSolverMomentumPropagation <= 0.0f && bAlwaysPropagate)
		{
			NodeQueue.Add(Node);
			Rigid->SetSleepCounter(int8(SleepCounterValue)); // Force reset in partial wake events to prevent immediate re-sleep
			if (!bIsAwake)
			{
				Node->Flags.bIsSleeping = false;
				WakeUpQueue.Add(Rigid);
			}
			return true;
		}

		// Compute the minimum momentum required to make the particle move significantly.
		// Approximation of the squared angular momentum using the smallest inertia diagonal in body-space
		// This gives us the lower bound of when to consider the particle to be moving.
		// linear: ||p||^2 = m^2 v^2
		// angular: ||L||^2 = w^T I^2 w ~= w^2 * I_min^2
		const FRealSingle LinearMomentumSq = FMath::Square(FRealSingle(Rigid->M())) * Node->WakeLinearThresholdSq;
		const FRealSingle AngularMomentumSq = FMath::Square(FRealSingle(Rigid->I().Min())) * Node->WakeAngularThresholdSq;

		// Other momentum needed to perform momentum propagation
		if (CVars::ChaosSolverMomentumPropagation > 0.0f && OtherMomentumSq)
		{
			const FRealSingle ReceivedLinearMomentumSq = OtherMomentumSq->Linear;
			const FRealSingle ReceivedAngularMomentumSq = OtherMomentumSq->Angular;

			if (bAlwaysPropagate
				|| ReceivedLinearMomentumSq > LinearMomentumSq
				|| ReceivedAngularMomentumSq > AngularMomentumSq)
			{
				// NOTE: Since we deal with the squared momentum, we also square the propagation discount factor.
				// @todo (Chaos): It would be great to use the direction of the momentum as opposed to the absolute value. 
				// For contacts, if the received momentum acts in the opposing direction of the relative position of the particle,
				// there is no need to propagate the wake through the particle.
				const FRealSingle ResidualLinearMomentumSq = ReceivedLinearMomentumSq * FMath::Square(CVars::ChaosSolverMomentumPropagation);
				const FRealSingle ResidualAngularMomentumSq = ReceivedAngularMomentumSq * FMath::Square(CVars::ChaosSolverMomentumPropagation);

				MomentumQueue.Add(FPBDParticleMomentumSq(ResidualLinearMomentumSq, ResidualAngularMomentumSq));
				NodeQueue.Add(Node);
				Rigid->SetSleepCounter(int8(SleepCounterValue)); // Force reset in partial wake events to prevent immediate re-sleep
				if (!bIsAwake)
				{
					Node->Flags.bIsSleeping = false;
					WakeUpQueue.Add(Rigid);
				}
				return true;
			}
		}
		return false;
	}

	void FPBDIslandManager::SyncConstraintSleepStateToParticles(FPBDIsland* Island)
	{
		auto SyncConstraintSleepStateToParticlesLambda = [](FPBDIslandConstraint* IslandEdge)
			{
				const FPBDIslandParticle* Node0 = IslandEdge->Nodes[0];
				const FPBDIslandParticle* Node1 = IslandEdge->Nodes[1];
				FConstraintHandle* Constraint = IslandEdge->GetConstraint();

				if (Node0 != nullptr && Node1 != nullptr)
				{
					const bool bIsSleeping0 = IsParticleSleeping(Node0->GetParticle());
					const bool bIsSleeping1 = IsParticleSleeping(Node1->GetParticle());
					const bool bIsDynamic0 = IsParticleDynamic(Node0->GetParticle());
					const bool bIsDynamic1 = IsParticleDynamic(Node1->GetParticle());

					// Deactivate constraint if:
					// Case (a) Both particles are sleeping
					// Case (b) One particle is sleeping and the other is not dynamic (i.e. kinematic, static, etc.)
					const bool bAreBothSleeping = bIsSleeping0 && bIsSleeping1;
					const bool bIsSleeping0Kinematic1 = bIsSleeping0 && !bIsDynamic1;
					const bool bIsSleeping1Kinematic0 = bIsSleeping1 && !bIsDynamic0;
					if (bAreBothSleeping || bIsSleeping0Kinematic1 || bIsSleeping1Kinematic0)
					{
						IslandEdge->Flags.bIsSleeping = true;
						Constraint->SetIsSleeping(true);
					}

					// Activate the constraint if:
					// Case (a) at least one particle is awake
					const bool bIsAwakeDynamic0 = !bIsSleeping0 && bIsDynamic0;
					const bool bIsAwakeDynamic1 = !bIsSleeping1 && bIsDynamic1;
					if (bIsAwakeDynamic0 || bIsAwakeDynamic1)
					{
						Constraint->SetIsSleeping(false);
						IslandEdge->Flags.bIsSleeping = false;
					}
				}
				else
				{
					// If one node is NULL, the edge sleep state should match the other node.
					if (Node0 != nullptr) // && Node1 == nullptr
					{
						IslandEdge->Flags.bIsSleeping = Node0->Flags.bIsSleeping;
						Constraint->SetIsSleeping(Node0->Flags.bIsSleeping);
					}
					else if (Node1 != nullptr) // && Node0 == nullptr
					{
						IslandEdge->Flags.bIsSleeping = Node1->Flags.bIsSleeping;
						Constraint->SetIsSleeping(Node1->Flags.bIsSleeping);
					}
					else // Node0 == nullptr && Node1 == nullptr
					{
						IslandEdge->Flags.bIsSleeping = false;
						Constraint->SetIsSleeping(false);
					}
				}
			};

		TArray<UE::Tasks::FTask> PendingEdgesTask;

		for (TArray<FPBDIslandConstraint*>& IslandEdges : Island->ContainerEdges)
		{
			int32 NumEdges = IslandEdges.Num();
			if (NumEdges > 0)
			{
				constexpr int32 MinEdgesPerTask = 100;
				const int32 NumTasks = FMath::Max(FMath::Min(FTaskGraphInterface::Get().GetNumWorkerThreads() - 1, Chaos::MaxNumWorkers), 1);
				const int32 NumEdgeTask = FMath::Max(FMath::Min(NumTasks, NumEdges), 1);
				const int32 EdgesByTask = FMath::Max(FMath::DivideAndRoundUp(NumEdges, NumEdgeTask), MinEdgesPerTask);
				const int32 NumEdgeBatches = FMath::DivideAndRoundUp(NumEdges, EdgesByTask);

				if (NumEdgeBatches == 1) // in series
				{
					for (FPBDIslandConstraint* IslandEdge : IslandEdges)
					{
						SyncConstraintSleepStateToParticlesLambda(IslandEdge);
					}
				}
				else // in parallel
				{
					for (int32 BatchIndex = 0; BatchIndex < NumEdgeBatches; BatchIndex++)
					{
						const int32 StartIndex = BatchIndex * EdgesByTask;
						int32 EndIndex = (BatchIndex + 1) * EdgesByTask;
						EndIndex = FMath::Min(NumEdges, EndIndex);

						UE::Tasks::FTask EdgeTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [StartIndex, EndIndex, &IslandEdges, SyncConstraintSleepStateToParticlesLambda]()
							{
								for (int32 Index = StartIndex; Index < EndIndex; ++Index)
								{
									FPBDIslandConstraint* IslandEdge = IslandEdges[Index];
									SyncConstraintSleepStateToParticlesLambda(IslandEdge);
								}
							});
						PendingEdgesTask.Add(EdgeTask);
					}
				}
			}
		}
		UE::Tasks::Wait(PendingEdgesTask);
	}

	void FPBDIslandManager::WakeUpParticles(TArray<FGeometryParticleHandle*>& ParticlesToWake, const bool bResetSleepCounter)
	{
		if (!ParticlesToWake.IsEmpty())
		{
			const auto UpdateShapeTransforms = [](const TArray<FGeometryParticleHandle*>& ParticlesToWake)
				{
					for (FGeometryParticleHandle* Particle : ParticlesToWake)
					{
						for (const TUniquePtr<FPerShapeData>& Shape : Particle->ShapesArray())
						{
							Shape->UpdateLeafWorldTransform(Particle);
						}
					}
				};

			Particles.ActivateParticles(ParticlesToWake);
			if (bResetSleepCounter)
			{
				for (FGeometryParticleHandle* Particle : ParticlesToWake)
				{
					if (FPBDRigidParticleHandle* Rigid = Particle->CastToRigidParticle())
					{
						Rigid->SetSleepCounter(0);
					}
				}
			}
			UpdateShapeTransforms(ParticlesToWake);
		}
	}

	void FPBDIslandManager::ResetParticleSleepCounter(FPBDIsland* Island)
	{
		for (FPBDIslandParticle* Node : Island->Nodes)
		{
			FPBDRigidParticleHandle* Rigid = Node->GetParticle()->CastToRigidParticle();
			if (Rigid != nullptr)
			{
				Rigid->SetSleepCounter(0);
			}
		}
	}

	void FPBDIslandManager::ResetEdgeFlags(FPBDIsland* Island)
	{
		for (TArray<FPBDIslandConstraint*>& IslandEdges : Island->ContainerEdges)
		{
			for (FPBDIslandConstraint* IslandEdge : IslandEdges)
			{
				IslandEdge->Flags.bIsNew = false;
				IslandEdge->Flags.bWasProcessed = false;
			}
		}
	}

	void FPBDIslandManager::ResetNodeFlags(FPBDIsland* Island)
	{
		for (FPBDIslandParticle* Node : Island->Nodes)
		{
			Node->Flags.bWasUpdated = false;
			Node->Flags.bWasProcessed = false;
			Node->Flags.bWasTransformSetExplicitly = false;
		}
	}

	void FPBDIslandManager::ResetEdgeFlags()
	{
		for (FPBDIslandConstraint* Edge : Edges)
		{
			Edge->Flags.bIsNew = false;
			Edge->Flags.bWasProcessed = false;
		}
	}

	void FPBDIslandManager::ResetNodeFlags()
	{
		for (FPBDIslandParticle* Node : Nodes)
		{
			Node->Flags.bWasUpdated = false;
			Node->Flags.bWasProcessed = false;
			Node->Flags.bWasTransformSetExplicitly = false;
		}
	}


	void FPBDIslandManager::ProcessDisable(TFunctionRef<void(FPBDRigidParticleHandle*)> ParticleDisableFunctor)
	{
		// @todo(chaos): parallelize
		if (DisableCounterThreshold == TNumericLimits<int32>::Max())
		{
			return;
		}

		TArray<FPBDRigidParticleHandle*> DisableParticles;

		// Check to see if islolated particles (not in an island) need to be disabled
		// We prefer to iterate over islands because nodes in the graph cache their thresholds
		ProcessParticlesDisable(DisableParticles);

		// Check islands to see if we want to disable the particles in them
		for (FPBDIsland* Island : Islands)
		{
			if (!Island->Flags.bIsSleeping && !Island->IsUsingCache())
			{
				// Update the sleep state based on particle movement etc
				ProcessIslandDisable(Island, DisableParticles);
			}
		}

		// Disable the particles we identified
		for (FPBDRigidParticleHandle* Particle : DisableParticles)
		{
			if (Particle != nullptr)
			{
				ParticleDisableFunctor(Particle);
			}
		}
	}

	void FPBDIslandManager::ProcessParticlesDisable(TArray<FPBDRigidParticleHandle*>& OutDisableParticles)
	{
		// Check all isolated particle (not in any island)
		for (FTransientPBDRigidParticleHandle& Particle : Particles.GetActiveDynamicMovingKinematicParticlesView())
		{
			if (Particle.IsDynamic() && !Particle.IsInConstraintGraph())
			{
				// @todo(chaos): this is very expensive becuase we have to search materials.
				// However this system is only used by Fields via PerParticlePhysicsMaterials, and not exposed to the editor.
				// Maybe we should just skip looking further if no PerParticlePhysicsMaterials is assigned.
				FRealSingle DisableLinearThreshold, DisableAngularThreshold;
				GetParticleDisableThresholds(Particle.Handle(), PhysicsMaterials, PerParticlePhysicsMaterials, SimMaterials, DisableLinearThreshold, DisableAngularThreshold);

				// Check for disable
				if ((DisableLinearThreshold > 0) || (DisableAngularThreshold > 0))
				{
					int32 DisableCounter = 0;

					const FReal VSq = Particle.VSmooth().SizeSquared();
					const FReal WSq = Particle.WSmooth().SizeSquared();
					if ((VSq < FMath::Square(DisableLinearThreshold)) && (WSq < FMath::Square(DisableAngularThreshold)))
					{
						DisableCounter = Particle.DisableCounter() + 1;
						if (DisableCounter > DisableCounterThreshold)
						{
							OutDisableParticles.Add(Particle.Handle());
						}
					}

					Particle.SetDisableCounter(int8(DisableCounter));
				}
			}
		}
	}

	void FPBDIslandManager::ProcessIslandDisable(FPBDIsland* Island, TArray<FPBDRigidParticleHandle*>& OutDisableParticles)
	{
		for (FPBDIslandParticle* Node : Island->GetParticles())
		{
			// All zeroes means never disable
			if ((Node->DisableLinearThresholdSq <= 0) && (Node->DisableAngularThresholdSq <= 0))
			{
				continue;
			}

			if (FPBDRigidParticleHandle* Particle = Node->GetParticle()->CastToRigidParticle())
			{
				int32 DisableCounter = 0;

				// Did we exceed the velocity thresholds?
				const FReal VSq = Particle->GetV().SizeSquared();
				const FReal WSq = Particle->GetW().SizeSquared();
				if ((VSq < Node->DisableLinearThresholdSq) && (WSq < Node->DisableAngularThresholdSq))
				{
					// We are within the velocity thresholds, so see if we should disable
					DisableCounter = Particle->DisableCounter() + 1;
					if (DisableCounter > DisableCounterThreshold)
					{
						OutDisableParticles.Add(Particle);
					}
				}

				Particle->SetDisableCounter(int8(DisableCounter));
			}
		}
	}

	void FPBDIslandManager::ApplyDeterminism()
	{
		// Nothing to do as long as we sorted constraints
	}

	bool FPBDIslandManager::Validate() const
	{
		SCOPE_CYCLE_COUNTER(STAT_IslandManager_Validate);

		if (!CVars::bChaosConstraintGraphValidate)
		{
			return true;
		}

		bool bIsValid = true;

		// Make sure we have processed the merges etc
		CHAOS_CONSTRAINTGRAPH_VALIDATE_TEST(MergeSets.IsEmpty());

		// Check various Node properties
		for (FPBDIslandParticle* Node : Nodes)
		{
			// All dynamic nodes should be in an island
			CHAOS_CONSTRAINTGRAPH_VALIDATE_TEST(!Node->Flags.bIsDynamic || (Node->Island != nullptr));

			// Kinematic nodes must not be in an island
			CHAOS_CONSTRAINTGRAPH_VALIDATE_TEST(Node->Flags.bIsDynamic || (Node->Island == nullptr));

			// Nodes without edges should have been removed
			// Actually kinematics with no edges may now remain in the graph until the
			// next UpdateParticles (i.e., the next tick) after their edges were removed
			CHAOS_CONSTRAINTGRAPH_VALIDATE_TEST(!Node->Flags.bIsDynamic || (Node->Edges.Num() > 0));

			if (Node->Island != nullptr)
			{
				// Make sure the island exists
				CHAOS_CONSTRAINTGRAPH_VALIDATE_TEST(Islands[Node->Island->ArrayIndex] == Node->Island);

				// Make sure the island knows about the node
				CHAOS_CONSTRAINTGRAPH_VALIDATE_TEST(Node->Island->Nodes.Contains(Node));
				CHAOS_CONSTRAINTGRAPH_VALIDATE_TEST(Node->Island->Nodes[Node->IslandArrayIndex] == Node);

				// Make sure the particle sleep state is correct
				if (Node->Island->Flags.bIsSleeping)
				{
					CHAOS_CONSTRAINTGRAPH_VALIDATE_TEST(Node->Particle->Sleeping())
				}
				CHAOS_CONSTRAINTGRAPH_VALIDATE_TEST(Node->Particle->Sleeping() == Node->Flags.bIsSleeping);
			}

			// Make sure the edge points to the node and that we're in the same island
			for (FPBDIslandConstraint* Edge : Node->Edges)
			{
				CHAOS_CONSTRAINTGRAPH_VALIDATE_TEST((Edge->Nodes[0] == Node) || (Edge->Nodes[1] == Node));

				if (Node->Flags.bIsDynamic)
				{
					CHAOS_CONSTRAINTGRAPH_VALIDATE_TEST(Node->Island == Edge->Island);
				}
			}

			// Particle sleep state should match
			CHAOS_CONSTRAINTGRAPH_VALIDATE_TEST(Node->Flags.bIsSleeping == IsParticleSleeping(Node->Particle));
		}

		// Check various Edge properties
		for (FPBDIslandConstraint* Edge : Edges)
		{
			// Make sure nodes know about the edge
			CHAOS_CONSTRAINTGRAPH_VALIDATE_TEST((Edge->Nodes[0] == nullptr) || Edge->Nodes[0]->Edges.Contains(Edge));
			CHAOS_CONSTRAINTGRAPH_VALIDATE_TEST((Edge->Nodes[1] == nullptr) || Edge->Nodes[1]->Edges.Contains(Edge));

			// All edges should be in an island (even inactive edges between two kinematics)
			const bool bIsInIsland = (Edge->Island != nullptr);
			CHAOS_CONSTRAINTGRAPH_VALIDATE_TEST(bIsInIsland);

			if (bIsInIsland)
			{
				// Make sure the island exists
				CHAOS_CONSTRAINTGRAPH_VALIDATE_TEST(Islands[Edge->Island->ArrayIndex] == Edge->Island);

				// Make sure the island edge index is correct
				CHAOS_CONSTRAINTGRAPH_VALIDATE_TEST(Edge->Island->ContainerEdges[Edge->ContainerIndex][Edge->IslandArrayIndex] == Edge);

				// Make sure the island and edge sleep states match
				if (!Edge->Island->IsPartialSleepEnabled() && Edge->Constraint->SupportsSleeping())
				{
					CHAOS_CONSTRAINTGRAPH_VALIDATE_TEST(Edge->Island->Flags.bIsSleeping == Edge->Flags.bIsSleeping);
				}

				// Make sure the sleep state matches between the edge and connected nodes
				if (Edge->Flags.bIsSleeping)
				{
					// The edge should be sleeping if it is connected to 2 non-moving nodes, i.e. dynamic, sleeping OR kinematic, not moving
					const bool bIsNode0NotMoving = (Edge->Nodes[0] != nullptr) ? Edge->Nodes[0]->Flags.bIsSleeping || !Edge->Nodes[0]->Flags.bIsMoving : true;
					const bool bIsNode1NotMoving = (Edge->Nodes[1] != nullptr) ? Edge->Nodes[1]->Flags.bIsSleeping || !Edge->Nodes[1]->Flags.bIsMoving : true;
					CHAOS_CONSTRAINTGRAPH_VALIDATE_TEST(bIsNode0NotMoving && bIsNode1NotMoving);
				}
				else
				{
					// The edge should be awake if it is connected to at least 1 non-sleeping dynamic or 1 moving kinematic.
					const bool bIsConnectedToNonSleeping = (Edge->Nodes[0] != nullptr ? !Edge->Nodes[0]->Flags.bIsSleeping : false) || (Edge->Nodes[1] != nullptr ? !Edge->Nodes[1]->Flags.bIsSleeping : false);
					const bool bIsConnectedToMovingKinematic = (Edge->Nodes[0] != nullptr ? Edge->Nodes[0]->Flags.bIsMoving : false) || (Edge->Nodes[1] != nullptr ? Edge->Nodes[1]->Flags.bIsMoving : false);
					CHAOS_CONSTRAINTGRAPH_VALIDATE_TEST(bIsConnectedToNonSleeping || bIsConnectedToMovingKinematic);
				}
			}

			// Constraint sleep state should match edge
			if (Edge->Constraint->SupportsSleeping())
			{
				CHAOS_CONSTRAINTGRAPH_VALIDATE_TEST(Edge->Flags.bIsSleeping == Edge->Constraint->IsSleeping());
			}

			// Make sure that flags were reset to false
			CHAOS_CONSTRAINTGRAPH_VALIDATE_TEST(!Edge->Flags.bIsNew);
			CHAOS_CONSTRAINTGRAPH_VALIDATE_TEST(!Edge->Flags.bWasProcessed);
		}

		// Check various Island properties
		for (FPBDIsland* Island : Islands)
		{
			// We are done with merges
			CHAOS_CONSTRAINTGRAPH_VALIDATE_TEST(Island->MergeSet == nullptr);

			// Island wake flag was reset
			CHAOS_CONSTRAINTGRAPH_VALIDATE_TEST(Island->Flags.bCheckSleep == false);

			// No dupes in the lists
			for (FPBDIslandParticle* Node0 : Island->Nodes)
			{
				int32 Count = 0;
				for (FPBDIslandParticle* Node1 : Island->Nodes)
				{
					if (Node0 == Node1)
					{
						++Count;
					}
				}
				CHAOS_CONSTRAINTGRAPH_VALIDATE_TEST(Count == 1);
			}
			for (TArray<FPBDIslandConstraint*> IslandContainerEdges : Island->ContainerEdges)
			{
				for (FPBDIslandConstraint* Edge0 : IslandContainerEdges)
				{
					int32 Count = 0;
					for (FPBDIslandConstraint* Edge1 : IslandContainerEdges)
					{
						if (Edge0 == Edge1)
						{
							++Count;
						}
					}
					CHAOS_CONSTRAINTGRAPH_VALIDATE_TEST(Count == 1);
				}
			}

			// Edge Count is correct
			int32 NumIslandEdges = 0;
			for (TArray<FPBDIslandConstraint*> IslandContainerEdges : Island->ContainerEdges)
			{
				NumIslandEdges += IslandContainerEdges.Num();
			}
			CHAOS_CONSTRAINTGRAPH_VALIDATE_TEST(NumIslandEdges == Island->NumEdges);
		}

		return true;
	}

	void FPBDIslandManager::UpdateAllParticleSleepMetrics(const FRealSingle Dt)
	{
		for (FTransientPBDRigidParticleHandle& Rigid : Particles.GetActiveDynamicMovingKinematicParticlesView())
		{
			UpdateParticleSleepMetrics(Rigid, Dt);
		}
	}

#if CHAOS_DEBUG_DRAW
	void FPBDIslandManager::DebugDrawSleepState(const DebugDraw::FChaosDebugDrawSettings* DebugDrawSettings) const
	{
		// Loop over isolated particles
		for (FTransientPBDRigidParticleHandle& Rigid : Particles.GetActiveDynamicMovingKinematicParticlesView())
		{
			if (Rigid.IsDynamic() && !Rigid.IsInConstraintGraph())
			{
				FColor Color = FColor(128, 128, 128);

				if (!Rigid.IsSleeping())
				{
					FRealSingle SleepLinearThreshold, SleepAngularThreshold;
					int32 SleepCounterThreshold;
					GetIsolatedParticleSleepThresholds(Rigid.Handle(), PhysicsMaterials, PerParticlePhysicsMaterials, SimMaterials, SleepLinearThreshold, SleepAngularThreshold, SleepCounterThreshold);

					// Check for sleep
					if ((SleepLinearThreshold > 0) || (SleepAngularThreshold > 0))
					{
						if (SleepCounterThreshold < TNumericLimits<int32>::Max())
						{
							// Isolated particles have a max sleep counter of 127 (to reduce counter space in the particle)
							SleepCounterThreshold = FMath::Min(SleepCounterThreshold, TNumericLimits<int8>::Max());

							// Did we exceed the velocity threshold?
							const bool bIsParticlePreventingSleep = ((Rigid.VSmooth().SizeSquared() > FMath::Square(SleepLinearThreshold)) || (Rigid.WSmooth().SizeSquared() > FMath::Square(SleepAngularThreshold)));
							Color = (bIsParticlePreventingSleep) ? FColor::Red : FColor::Green;
						}
					}
				}

				DebugDraw::DrawParticleShapes(FRigidTransform3::Identity, Rigid.Handle(), Color, DebugDrawSettings);
			}
		}

		// Loop over particles in islands
		for (FPBDIsland* Island : Islands)
		{
			const FColor SleepingColor = FColor::Black;

			if (Island->Flags.bIsSleeping)
			{
				for (FPBDIslandParticle* Node : Island->GetParticles())
				{
					DebugDraw::DrawParticleShapes(FRigidTransform3::Identity, Node->GetParticle(), SleepingColor, DebugDrawSettings);
				}
				continue;
			}

			for (FPBDIslandParticle* Node : Island->GetParticles())
			{
				FColor Color = FColor(128, 128, 128);

				// All zeroes means never sleep/disable
				if ((Node->SleepLinearThresholdSq > 0) || (Node->SleepAngularThresholdSq > 0))
				{
					// Check the particle state against the thresholds
					if (FPBDRigidParticleHandle* Rigid = Node->GetParticle()->CastToRigidParticle())
					{
						if (Rigid->Sleeping())
						{
							Color = SleepingColor;
						}
						else
						{
							// Did we exceed the velocity threshold?
							const bool bIsParticlePreventingSleep = ((Rigid->VSmooth().SizeSquared() > Node->SleepLinearThresholdSq) || (Rigid->WSmooth().SizeSquared() > Node->SleepAngularThresholdSq));
							Color = (bIsParticlePreventingSleep) ? FColor::Red : FColor::Green;
						}
					}
				}

				DebugDraw::DrawParticleShapes(FRigidTransform3::Identity, Node->GetParticle(), Color, DebugDrawSettings);
			}
		}
	}
#endif

} // namsepace Chaos::Private
