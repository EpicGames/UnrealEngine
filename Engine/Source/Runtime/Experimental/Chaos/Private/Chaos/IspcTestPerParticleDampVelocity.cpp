// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS && INTEL_ISPC

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"
#include "Chaos/PerParticleDampVelocity.h"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FIspcPerParticleDampVelocity, "Ispc.Physics.PerParticleDampVelocity", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FIspcPerParticleDampVelocity::RunTest(const FString& Parameters)
{
	using namespace Chaos;	
	
	// Chaos module doesn't let us include Engine.h, therefore we cannot call the console command.
	bool InitialState = bChaos_DampVelocity_ISPC_Enabled;

	const int32 ParticleOffset = 0;
	const int32 NumParticles = 101;

	const Softs::FSolverReal InvMValue = 1.f;
	const Softs::FSolverReal MValue = 1.f;
	Softs::FPAndInvM FPInvMData;
	FPInvMData.P = Softs::FSolverVec3(0.f);
	FPInvMData.InvM = InvMValue;

	Softs::FSolverVec3 XSolverVec(1.f, 2.f, 3.f);
	Softs::FSolverVec3 VSolverVec(1.f, 2.f, 3.f);

	bChaos_DampVelocity_ISPC_Enabled = true;
	Softs::FSolverParticles ISPCParticles;
	// TParticles::Resize() will make all the necessary arrays the same size.
	ISPCParticles.Resize(NumParticles);
	{
		// HACK: Can't get to TDynamicParticles::MInvM any other way.
		TArray<Softs::FSolverReal>& InvMArray = *const_cast<TArrayCollectionArray<Softs::FSolverReal>*>(&ISPCParticles.GetInvM());
		TArray<Softs::FPAndInvM>& FPInvMArray = ISPCParticles.GetPAndInvM();
		TArray<Softs::FSolverVec3>& XArray = ISPCParticles.AllX();
		TArray<Softs::FSolverVec3>& VArray = *const_cast<TArrayCollectionArray<Softs::FSolverVec3>*>(&ISPCParticles.GetV());
		TArray<Softs::FSolverReal>& MArray = *const_cast<TArrayCollectionArray<Softs::FSolverReal>*>(&ISPCParticles.GetM());

		for (int32 i = 0; i < NumParticles; ++i)
		{
			InvMArray[i] = InvMValue;
			FPInvMArray[i].P = FPInvMData.P;
			FPInvMArray[i].InvM = FPInvMData.InvM;
			XArray[i] = XSolverVec;
			VArray[i] = VSolverVec;
			MArray[i] = MValue;
		}

		Softs::FPerParticleDampVelocity PerParticleDampVelocity;
		PerParticleDampVelocity.UpdatePositionBasedState(ISPCParticles, 0, NumParticles);
	}

	bChaos_DampVelocity_ISPC_Enabled = false;
	Softs::FSolverParticles CPPParticles;
	// TParticles::Resize() will make all the necessary arrays the same size.
	CPPParticles.Resize(NumParticles);
	{
		// HACK: Can't get to TDynamicParticles::MInvM any other way.
		TArray<Softs::FSolverReal>& InvMArray = *const_cast<TArrayCollectionArray<Softs::FSolverReal>*>(&CPPParticles.GetInvM());
		TArray<Softs::FPAndInvM>& FPInvMArray = CPPParticles.GetPAndInvM();
		TArray<Softs::FSolverVec3>& XArray = CPPParticles.AllX();
		TArray<Softs::FSolverVec3>& VArray = *const_cast<TArrayCollectionArray<Softs::FSolverVec3>*>(&CPPParticles.GetV());
		TArray<Softs::FSolverReal>& MArray = *const_cast<TArrayCollectionArray<Softs::FSolverReal>*>(&CPPParticles.GetM());

		for (int32 i = 0; i < NumParticles; ++i)
		{
			InvMArray[i] = InvMValue;
			FPInvMArray[i].P = FPInvMData.P;
			FPInvMArray[i].InvM = FPInvMData.InvM;
			XArray[i] = XSolverVec;
			VArray[i] = VSolverVec;
			MArray[i] = MValue;
		}

		Softs::FPerParticleDampVelocity PerParticleDampVelocity;
		PerParticleDampVelocity.UpdatePositionBasedState(CPPParticles, 0, NumParticles);
	}

	bChaos_DampVelocity_ISPC_Enabled = InitialState;

	TArray<Softs::FPAndInvM>& ISPCFPInvMArray = ISPCParticles.GetPAndInvM();
	TArray<Softs::FPAndInvM>& CPPFPInvMArray = CPPParticles.GetPAndInvM();
	for (int32 i = 0; i < ISPCFPInvMArray.Num(); ++i)
	{
		const Softs::FPAndInvM& ISPCElement = ISPCFPInvMArray[i];
		const Softs::FPAndInvM& CPPElement = CPPFPInvMArray[i];

		const FString PMessage(FString::Format(TEXT("P Vec {0}"), { i }));
		const FString InvMMessage(FString::Format(TEXT("InvM {0}"), { i }));
		TestTrue(*PMessage, ISPCElement.P.Equals(CPPElement.P));
		TestEqual(*InvMMessage, ISPCElement.InvM, CPPElement.InvM, SMALL_NUMBER);
	}

	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS
