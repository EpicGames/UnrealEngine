// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/Joint/ColoringGraph.h"

//PRAGMA_DISABLE_OPTIMIZATION

namespace Chaos
{
	void FColoringGraph::Islandize()
	{
		Islands.Reset();

		for (int32 VertexIndex = 0; VertexIndex < Vertices.Num(); ++VertexIndex)
		{
			int32 IslandIndex = Islands.AddDefaulted();
			Vertices[VertexIndex].Island = IslandIndex;
			Islands[IslandIndex].VertexIndices.Add(VertexIndex);
		}

		for (int32 EdgeIndex = 0; EdgeIndex < Edges.Num(); ++EdgeIndex)
		{
			int32 VertexIndex0 = Edges[EdgeIndex].VertexIndices[0];
			int32 VertexIndex1 = Edges[EdgeIndex].VertexIndices[1];
			int32 IslandIndex0 = Vertices[VertexIndex0].Island;
			int32 IslandIndex1 = Vertices[VertexIndex1].Island;
			if (IslandIndex0 != IslandIndex1)
			{
				MergeIslands(IslandIndex0, IslandIndex1);
			}
		}

		PackIslands();
	}

	void FColoringGraph::Levelize()
	{
		TArray<int32> LevelQueue;
		LevelQueue.Reserve(Vertices.Num());
		for (int32 VertexIndex = 0; VertexIndex < Vertices.Num(); ++VertexIndex)
		{
			if (Vertices[VertexIndex].Level == 0)
			{
				LevelQueue.Add(VertexIndex);
			}
		}

		for (int32 LevelQueueIndex = 0; LevelQueueIndex < LevelQueue.Num(); ++LevelQueueIndex)
		{
			int32 VertexIndex = LevelQueue[LevelQueueIndex];
			int32 VertexLevel = Vertices[VertexIndex].Level;

			for (int32 EdgeIndex : Vertices[VertexIndex].EdgeIndices)
			{
				int32 ChildVertexIndex0 = Edges[EdgeIndex].VertexIndices[0];
				int32 ChildVertexIndex1 = Edges[EdgeIndex].VertexIndices[1];
				if (Vertices[ChildVertexIndex0].Level == INDEX_NONE)
				{
					Vertices[ChildVertexIndex0].Level = VertexLevel + 1;
					LevelQueue.Add(ChildVertexIndex0);
				}
				if (Vertices[ChildVertexIndex1].Level == INDEX_NONE)
				{
					Vertices[ChildVertexIndex1].Level = VertexLevel + 1;
					LevelQueue.Add(ChildVertexIndex1);
				}
			}
		}
	}

	void FColoringGraph::Colorize()
	{
		// Get the Vertices sorted by level for better coloring result
		TArray<int32> VertexIndices;
		VertexIndices.Reserve(Vertices.Num());
		for (int32 VertexIndex = 0; VertexIndex < Vertices.Num(); ++VertexIndex)
		{
			VertexIndices.Add(VertexIndex);
		}

		VertexIndices.StableSort([this](int32 L, int32 R)
			{
				int32 LevelL = Vertices[L].Level;
				int32 LevelR = Vertices[R].Level;
				return LevelL < LevelR;
			});


		for (int32 VertexIndex : VertexIndices)
		{
			AssignColor(VertexIndex);
		}
	}

	void FColoringGraph::MergeIslands(int32 IslandIndex0, int32 IslandIndex1)
	{
		// We will put the Vertices from the smaller island into the larger one
		FIsland* Island0 = &Islands[IslandIndex0];
		FIsland* Island1 = &Islands[IslandIndex1];
		if (Island1->VertexIndices.Num() > Island0->VertexIndices.Num())
		{
			Swap(Island0, Island1);
			Swap(IslandIndex0, IslandIndex1);
		}

		if (Island1->VertexIndices.Num() > 0)
		{
			// Add Vertices to the first island
			Island0->VertexIndices.Append(Island1->VertexIndices);

			// Update the Vertices' island indices
			for (int32 VertexIndex1 : Island1->VertexIndices)
			{
				Vertices[VertexIndex1].Island = IslandIndex0;
			}

			// Remove Vertices from the second island
			Island1->VertexIndices.Empty();
		}
	}

	void FColoringGraph::PackIslands()
	{
		// Sort islands by number of vertices
		Islands.Sort([](const FIsland& L, const FIsland& R)
			{
				return L.VertexIndices.Num() > R.VertexIndices.Num();
			});

		// Fix all island indices and clip array to remove empty islands
		for (int32 IslandIndex = 0; IslandIndex < Islands.Num(); ++IslandIndex)
		{
			// When we hit an empty island we are done
			if (Islands[IslandIndex].VertexIndices.Num() == 0)
			{
				Islands.SetNum(IslandIndex);
				break;
			}

			// Set islands index on all verts in the island
			for (int32 VertexIndex : Islands[IslandIndex].VertexIndices)
			{
				Vertices[VertexIndex].Island = IslandIndex;
			}
		}
	}

	void FColoringGraph::AssignColor(int32 VertexIndex)
	{
		if (Vertices[VertexIndex].Color == INDEX_NONE)
		{
			int32 IslandIndex = Vertices[VertexIndex].Island;

			// We cannot use any color already used by our neighbors
			// @todo(ccaulfield): eliminate allocations
			TArray<bool> UsedColors;
			UsedColors.SetNumZeroed(Islands[IslandIndex].NextColorIndex);
			for (int32 EdgeIndex : Vertices[VertexIndex].EdgeIndices)
			{
				int32 OtherVertexIndex = (Edges[EdgeIndex].VertexIndices[0] == VertexIndex) ? Edges[EdgeIndex].VertexIndices[1] : Edges[EdgeIndex].VertexIndices[0];
				int32 OtherColorIndex = Vertices[OtherVertexIndex].Color;
				if (OtherColorIndex != INDEX_NONE)
				{
					UsedColors[OtherColorIndex] = true;
				}
			}

			// Find the first unused Color
			int32 VertexColor = INDEX_NONE;
			for (int32 ColorIndex = 0; ColorIndex < UsedColors.Num(); ++ColorIndex)
			{
				if (!UsedColors[ColorIndex])
				{
					VertexColor = ColorIndex;
				}
			}

			// If all colors are used, assign a new one
			if (VertexColor == INDEX_NONE)
			{
				VertexColor = Islands[IslandIndex].NextColorIndex++;
			}

			Vertices[VertexIndex].Color = VertexColor;
		}
	}

}