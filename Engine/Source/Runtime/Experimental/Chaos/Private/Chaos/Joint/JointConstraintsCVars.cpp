// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/Joint/JointConstraintsCVars.h"
#include "HAL/IConsoleManager.h"

#if INTEL_ISPC && !UE_BUILD_SHIPPING
bool bChaos_Joint_ISPC_Enabled = CHAOS_JOINT_SOLVER_ISPC_ENABLED_DEFAULT;
static FAutoConsoleVariableRef CVarChaosJointISPCEnabled(TEXT("p.Chaos.Joint.ISPC"), bChaos_Joint_ISPC_Enabled, TEXT("Whether to use ISPC optimizations in the Joint Solver"));
#endif

float Chaos_Joint_DegenerateRotationLimit = -0.998f;	// Cos(176deg)
FAutoConsoleVariableRef CVarChaosJointDegenerateRotationLimit(TEXT("p.Chaos.Joint.DegenerateRotationLimit"), Chaos_Joint_DegenerateRotationLimit, TEXT("Cosine of the swing angle that is considered degerenerate (default Cos(176deg))"));

float Chaos_Joint_VelProjectionAlpha = 0.1f;
FAutoConsoleVariableRef CVarChaosJointVelProjectionScale(TEXT("p.Chaos.Joint.VelProjectionAlpha"), Chaos_Joint_VelProjectionAlpha, TEXT("How much of the velocity correction to apply during projection. Equivalent to (1-damping) for projection velocity delta"));

bool bChaos_Joint_DisableSoftLimits = false;
FAutoConsoleVariableRef CVarChaosJointDisableSoftLimits(TEXT("p.Chaos.Joint.DisableSoftLimits"), bChaos_Joint_DisableSoftLimits, TEXT("Disable soft limits (for debugging only)"));

bool bChaos_Joint_Plasticity_ClampToLimits = true;
FAutoConsoleVariableRef CVarChaosJointPlasticityClampToLimits(TEXT("p.Chaos.Joint.Plasticity.ClampToLimits"), bChaos_Joint_Plasticity_ClampToLimits, TEXT("Clamp drive position targets to defined limits after plasticity computation"));

float Chaos_Joint_LinearVelocityThresholdToApplyRestitution = 1e-2f;
FAutoConsoleVariableRef CVarChaosJointLinearVelocityThresholdToApplyRestitution(TEXT("p.Chaos.Joint.LinearVelocityThresholdToApplyRestitution"), Chaos_Joint_LinearVelocityThresholdToApplyRestitution, TEXT("Apply restitution only if initial velocity is higher than this threshold (used in Quasipbd)"));

float Chaos_Joint_AngularVelocityThresholdToApplyRestitution = 1e-2f;
FAutoConsoleVariableRef CVarChaosJointAngularVelocityThresholdToApplyRestitution(TEXT("p.Chaos.Joint.AngularVelocityThresholdToApplyRestitution"), Chaos_Joint_AngularVelocityThresholdToApplyRestitution, TEXT("Apply restitution only if initial velocity is higher than this threshold (used in Quasipbd)"));

