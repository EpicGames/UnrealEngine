// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#if !defined(CHAOS_JOINT_SOLVER_ISPC_ENABLED_DEFAULT)
#define CHAOS_JOINT_SOLVER_ISPC_ENABLED_DEFAULT 0
#endif

// Support run-time toggling on supported platforms in non-shipping configurations
#if !INTEL_ISPC || UE_BUILD_SHIPPING
static constexpr bool bChaos_Joint_ISPC_Enabled = INTEL_ISPC && CHAOS_JOINT_SOLVER_ISPC_ENABLED_DEFAULT;
#else
extern bool bChaos_Joint_ISPC_Enabled;
#endif

extern bool bChaos_Joint_EarlyOut_Enabled;

extern float Chaos_Joint_DegenerateRotationLimit;

extern float Chaos_Joint_VelProjectionAlpha;

extern bool bChaos_Joint_DisableSoftLimits;

extern bool bChaos_Joint_Plasticity_ClampToLimits;

extern float Chaos_Joint_LinearVelocityThresholdToApplyRestitution;

extern float Chaos_Joint_AngularVelocityThresholdToApplyRestitution;

extern bool bChaos_Joint_UseCachedSolver;
