// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/Joint/JointSolverConstraints.h"


#include "Chaos/Joint/ChaosJointLog.h"
#include "Chaos/PBDJointConstraintTypes.h"
#include "Chaos/PBDJointConstraintUtilities.h"
#include "Chaos/Utilities.h"

#include "HAL/IConsoleManager.h"

//PRAGMA_DISABLE_OPTIMIZATION

int32 bChaos_Joint_MultiDimension = true;
FAutoConsoleVariableRef CVarChaosJointMultiDimension(TEXT("p.Chaos.Joint.MultiDimension"), bChaos_Joint_MultiDimension, TEXT(""));

namespace Chaos
{

	//
	//////////////////////////////////////////////////////////////////////////
	//

	FJointSolverConstraints::FJointSolverConstraints()
		: JointIndex(INDEX_NONE)
		, LinearRowIndexBegin(INDEX_NONE)
		, LinearRowIndexEnd(INDEX_NONE)
		, AngularRowIndexBegin(INDEX_NONE)
		, AngularRowIndexEnd(INDEX_NONE)
		, PositionTolerance(0.0f)
		, AngleTolerance(0.0f)
		, bNeedSwingTwist(false)
		, bNeedLockedAxes(false)
	{
	}

	void FJointSolverConstraints::AddPositionConstraints(
		TArray<FJointSolverConstraintRowData>& RowDatas,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		PositionTolerance = SolverSettings.PositionTolerance;

		LinearRowIndexBegin = RowDatas.Num();
		LinearRowIndexEnd = LinearRowIndexBegin;

		const TVector<EJointMotionType, 3>& LinearMotion = JointSettings.LinearMotionTypes;
		const TVector<bool, 3> bLinearLocked =
		{
			(LinearMotion[0] == EJointMotionType::Locked),
			(LinearMotion[1] == EJointMotionType::Locked),
			(LinearMotion[2] == EJointMotionType::Locked),
		};
		const TVector<bool, 3> bLinearLimted =
		{
			(LinearMotion[0] == EJointMotionType::Limited),
			(LinearMotion[1] == EJointMotionType::Limited),
			(LinearMotion[2] == EJointMotionType::Limited),
		};

		if (bLinearLocked[0] && bLinearLocked[1] && bLinearLocked[2])
		{
			// Hard point constraint (most common case)
			AddPointPositionConstraint(RowDatas, SolverSettings, JointSettings);
		}
		else if (bLinearLimted[0] && bLinearLimted[1] && bLinearLimted[2])
		{
			// Spherical constraint
			AddSphericalPositionConstraint(RowDatas, SolverSettings, JointSettings);
		}
		else if (bLinearLocked[1] && bLinearLocked[2] && !bLinearLocked[0])
		{
			// Line constraint along X axis
			AddCylindricalPositionConstraint(RowDatas, 0, LinearMotion[0], EJointMotionType::Locked, SolverSettings, JointSettings);
		}
		else if (bLinearLocked[0] && bLinearLocked[2] && !bLinearLocked[1])
		{
			// Line constraint along Y axis
			AddCylindricalPositionConstraint(RowDatas, 1, LinearMotion[1], EJointMotionType::Locked, SolverSettings, JointSettings);
		}
		else if (bLinearLocked[0] && bLinearLocked[1] && !bLinearLocked[2])
		{
			// Line constraint along Z axis
			AddCylindricalPositionConstraint(RowDatas, 2, LinearMotion[2], EJointMotionType::Locked, SolverSettings, JointSettings);
		}
		else if (bLinearLimted[1] && bLinearLimted[2] && !bLinearLimted[0])
		{
			// Cylindrical constraint along X axis
			AddCylindricalPositionConstraint(RowDatas, 0, LinearMotion[0], EJointMotionType::Limited, SolverSettings, JointSettings);
		}
		else if (bLinearLimted[0] && bLinearLimted[2] && !bLinearLimted[1])
		{
			// Cylindrical constraint along Y axis
			AddCylindricalPositionConstraint(RowDatas, 1, LinearMotion[1], EJointMotionType::Limited, SolverSettings, JointSettings);
		}
		else if (bLinearLimted[0] && bLinearLimted[1] && !bLinearLimted[2])
		{
			// Cylindrical constraint along Z axis
			AddCylindricalPositionConstraint(RowDatas, 2, LinearMotion[2], EJointMotionType::Limited, SolverSettings, JointSettings);
		}
		else if (bLinearLocked[0] || bLinearLimted[0])
		{
			// Planar constraint along X axis
			AddPlanarPositionConstraint(RowDatas, 0, LinearMotion[0], SolverSettings, JointSettings);
		}
		else if (bLinearLocked[1] || bLinearLimted[1])
		{
			// Planar constraint along Y axis
			AddPlanarPositionConstraint(RowDatas, 1, LinearMotion[1], SolverSettings, JointSettings);
		}
		else if (bLinearLocked[2] || bLinearLimted[2])
		{
			// Planar constraint along Z axis
			AddPlanarPositionConstraint(RowDatas, 2, LinearMotion[2], SolverSettings, JointSettings);
		}

		if (SolverSettings.bEnableDrives)
		{
			TVector<bool, 3> bDriven =
			{
				(JointSettings.bLinearPositionDriveEnabled[0] || JointSettings.bLinearVelocityDriveEnabled[0]) && (JointSettings.LinearMotionTypes[0] != EJointMotionType::Locked),
				(JointSettings.bLinearPositionDriveEnabled[1] || JointSettings.bLinearVelocityDriveEnabled[1]) && (JointSettings.LinearMotionTypes[1] != EJointMotionType::Locked),
				(JointSettings.bLinearPositionDriveEnabled[2] || JointSettings.bLinearVelocityDriveEnabled[2]) && (JointSettings.LinearMotionTypes[2] != EJointMotionType::Locked),
			};

			if (bDriven[0] && bDriven[1] && bDriven[2])
			{
				AddPositionDrive(RowDatas, SolverSettings, JointSettings, EJointSolverConstraintUpdateType::Linear_SphericalDrive, INDEX_NONE);
			}
			else if (bDriven[1] && bDriven[2])
			{
				AddPositionDrive(RowDatas, SolverSettings, JointSettings, EJointSolverConstraintUpdateType::Linear_CircularDrive, 0);
			}
			else if (bDriven[0] && bDriven[2])
			{
				AddPositionDrive(RowDatas, SolverSettings, JointSettings, EJointSolverConstraintUpdateType::Linear_CircularDrive, 1);
			}
			else if (bDriven[0] && bDriven[1])
			{
				AddPositionDrive(RowDatas, SolverSettings, JointSettings, EJointSolverConstraintUpdateType::Linear_CircularDrive, 2);
			}
			else if (bDriven[0])
			{
				AddPositionDrive(RowDatas, SolverSettings, JointSettings, EJointSolverConstraintUpdateType::Linear_AxialDrive, 0);
			}
			else if (bDriven[1])
			{
				AddPositionDrive(RowDatas, SolverSettings, JointSettings, EJointSolverConstraintUpdateType::Linear_AxialDrive, 1);
			}
			else if (bDriven[2])
			{
				AddPositionDrive(RowDatas, SolverSettings, JointSettings, EJointSolverConstraintUpdateType::Linear_AxialDrive, 2);
			}
		}
	}

	void FJointSolverConstraints::UpdatePositionConstraints(
		const TArray<FJointSolverConstraintRowData>& RowDatas,
		TArray<FJointSolverConstraintRowState>& RowStates,
		const FJointSolverJointState& JointState,
		const FPBDJointSettings& JointSettings)
	{
		const FRotation3 R0 = JointState.Rs[0];
		const FVec3& X0 = JointState.Xs[0];
		const FVec3& X1 = JointState.Xs[1];

		// NOTE: second loop may jump rows, so reset all here
		for (int32 RowIndex = LinearRowIndexBegin; RowIndex < LinearRowIndexEnd; ++RowIndex)
		{
			RowStates[RowIndex].IterationReset();
		}

		// Calculate all the rotation constraint axes etc. Some of the update functions actually
		// update multple rows so we need to skip them in the loop if so.
		for (int32 RowIndex = LinearRowIndexBegin; RowIndex < LinearRowIndexEnd; /* no op */)
		{
			switch (RowDatas[RowIndex].UpdateType)
			{
			case EJointSolverConstraintUpdateType::Linear_Point:
				RowIndex += UpdatePointPositionConstraint(RowIndex, RowDatas, RowStates, JointSettings, X0, X1);
				break;
			case EJointSolverConstraintUpdateType::Linear_Spherical:
				RowIndex += UpdateSphericalPositionConstraint(RowIndex, RowDatas, RowStates, JointSettings, X0, X1);
				break;
			case EJointSolverConstraintUpdateType::Linear_Cylindrical:
				RowIndex += UpdateCylindricalPositionConstraint(RowIndex, RowDatas, RowStates, JointSettings, R0, X0, X1);
				break;
			case EJointSolverConstraintUpdateType::Linear_Planar:
				RowIndex += UpdatePlanarPositionConstraint(RowIndex, RowDatas, RowStates, JointSettings, R0, X0, X1);
				break;
			case EJointSolverConstraintUpdateType::Linear_SphericalDrive:
				RowIndex += UpdateSphericalPositionDrive(RowIndex, RowDatas, RowStates, JointSettings, R0, X0, X1);
				break;
			case EJointSolverConstraintUpdateType::Linear_CircularDrive:
				RowIndex += UpdateCircularPositionDrive(RowIndex, RowDatas, RowStates, JointSettings, R0, X0, X1);
				break;
			case EJointSolverConstraintUpdateType::Linear_AxialDrive:
				RowIndex += UpdateAxialPositionDrive(RowIndex, RowDatas, RowStates, JointSettings, R0, X0, X1);
				break;
			}
		}
	}

	void FJointSolverConstraints::AddRotationConstraints(
		TArray<FJointSolverConstraintRowData>& RowDatas,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		AngleTolerance = SolverSettings.AngleTolerance;

		AngularRowIndexBegin = RowDatas.Num();
		AngularRowIndexEnd = AngularRowIndexBegin;

		EJointMotionType TwistMotion = JointSettings.AngularMotionTypes[(int32)EJointAngularConstraintIndex::Twist];
		EJointMotionType Swing1Motion = JointSettings.AngularMotionTypes[(int32)EJointAngularConstraintIndex::Swing1];
		EJointMotionType Swing2Motion = JointSettings.AngularMotionTypes[(int32)EJointAngularConstraintIndex::Swing2];

		// Add twist constraint
		if (SolverSettings.bEnableTwistLimits)
		{
			bool bTwistSoft = JointSettings.bSoftTwistLimitsEnabled;
			if (TwistMotion == EJointMotionType::Limited)
			{
				AddTwistConstraint(RowDatas, SolverSettings, JointSettings, EJointSolverConstraintUpdateType::Angular_Twist, bTwistSoft);
			}
			else if (TwistMotion == EJointMotionType::Locked)
			{
				AddTwistConstraint(RowDatas, SolverSettings, JointSettings, EJointSolverConstraintUpdateType::Angular_Locked, false);
			}
		}

		// Add swing constraints
		if (SolverSettings.bEnableSwingLimits)
		{
			bool bSwingSoft = JointSettings.bSoftSwingLimitsEnabled;
			if ((Swing1Motion == EJointMotionType::Limited) && (Swing2Motion == EJointMotionType::Limited))
			{
				AddSwingConstraint(RowDatas, SolverSettings, JointSettings, EJointSolverConstraintUpdateType::Angular_Cone, EJointAngularConstraintIndex::Swing1, bSwingSoft);
			}
			else if ((Swing1Motion == EJointMotionType::Limited) && (Swing2Motion == EJointMotionType::Locked))
			{
				AddSwingConstraint(RowDatas, SolverSettings, JointSettings, EJointSolverConstraintUpdateType::Angular_SingleLockedSwing, EJointAngularConstraintIndex::Swing2, false);
				AddSwingConstraint(RowDatas, SolverSettings, JointSettings, EJointSolverConstraintUpdateType::Angular_SingleLimitedSwing, EJointAngularConstraintIndex::Swing1, bSwingSoft);
			}
			else if ((Swing1Motion == EJointMotionType::Limited) && (Swing2Motion == EJointMotionType::Free))
			{
				AddSwingConstraint(RowDatas, SolverSettings, JointSettings, EJointSolverConstraintUpdateType::Angular_DualConeSwing, EJointAngularConstraintIndex::Swing1, bSwingSoft);
			}
			else if ((Swing1Motion == EJointMotionType::Locked) && (Swing2Motion == EJointMotionType::Limited))
			{
				AddSwingConstraint(RowDatas, SolverSettings, JointSettings, EJointSolverConstraintUpdateType::Angular_SingleLockedSwing, EJointAngularConstraintIndex::Swing1, false);
				AddSwingConstraint(RowDatas, SolverSettings, JointSettings, EJointSolverConstraintUpdateType::Angular_SingleLimitedSwing, EJointAngularConstraintIndex::Swing2, bSwingSoft);
			}
			else if ((Swing1Motion == EJointMotionType::Locked) && (Swing2Motion == EJointMotionType::Locked))
			{
				AddSwingConstraint(RowDatas, SolverSettings, JointSettings, EJointSolverConstraintUpdateType::Angular_Locked, EJointAngularConstraintIndex::Swing1, false);
				AddSwingConstraint(RowDatas, SolverSettings, JointSettings, EJointSolverConstraintUpdateType::Angular_Locked, EJointAngularConstraintIndex::Swing2, false);
			}
			else if ((Swing1Motion == EJointMotionType::Locked) && (Swing2Motion == EJointMotionType::Free))
			{
				AddSwingConstraint(RowDatas, SolverSettings, JointSettings, EJointSolverConstraintUpdateType::Angular_SingleLockedSwing, EJointAngularConstraintIndex::Swing1, false);
			}
			else if ((Swing1Motion == EJointMotionType::Free) && (Swing2Motion == EJointMotionType::Limited))
			{
				AddSwingConstraint(RowDatas, SolverSettings, JointSettings, EJointSolverConstraintUpdateType::Angular_DualConeSwing, EJointAngularConstraintIndex::Swing2, bSwingSoft);
			}
			else if ((Swing1Motion == EJointMotionType::Free) && (Swing2Motion == EJointMotionType::Locked))
			{
				AddSwingConstraint(RowDatas, SolverSettings, JointSettings, EJointSolverConstraintUpdateType::Angular_SingleLockedSwing, EJointAngularConstraintIndex::Swing2, false);
			}
		}

		// Add Drives
		if (SolverSettings.bEnableDrives)
		{
			bool bTwistLocked = TwistMotion == EJointMotionType::Locked;
			bool bSwing1Locked = Swing1Motion == EJointMotionType::Locked;
			bool bSwing2Locked = Swing2Motion == EJointMotionType::Locked;

			// No SLerp drive if we have a locked rotation (it will be grayed out in the editor in this case, but could still have been set before the rotation was locked)
			if ((JointSettings.bAngularSLerpPositionDriveEnabled || JointSettings.bAngularSLerpVelocityDriveEnabled) && !bTwistLocked && !bSwing1Locked && !bSwing2Locked)
			{
				AddSLerpDrive(RowDatas, SolverSettings, JointSettings);
			}
			else
			{
				if ((JointSettings.bAngularTwistPositionDriveEnabled || JointSettings.bAngularTwistVelocityDriveEnabled) && !bTwistLocked)
				{
					AddTwistDrive(RowDatas, SolverSettings, JointSettings);
				}

				const bool bSwingDriveEnabled = (JointSettings.bAngularSwingPositionDriveEnabled || JointSettings.bAngularSwingVelocityDriveEnabled);
				if (bSwingDriveEnabled && !bSwing1Locked && !bSwing2Locked)
				{
					AddSwingDrive(RowDatas, SolverSettings, JointSettings, EJointSolverConstraintUpdateType::Angular_ConeDrive, EJointAngularConstraintIndex::Swing1);
				}
				else if (bSwingDriveEnabled && !bSwing1Locked)
				{
					AddSwingDrive(RowDatas, SolverSettings, JointSettings, EJointSolverConstraintUpdateType::Angular_SwingDrive, EJointAngularConstraintIndex::Swing1);
				}
				else if (bSwingDriveEnabled && !bSwing2Locked)
				{
					AddSwingDrive(RowDatas, SolverSettings, JointSettings, EJointSolverConstraintUpdateType::Angular_SwingDrive, EJointAngularConstraintIndex::Swing2);
				}
			}
		}

		// Determine what intermediate data needs to be calculated in the update step
		bNeedLockedAxes = false;
		bNeedSwingTwist = false;
		for (int32 RowIndex = AngularRowIndexBegin; RowIndex < AngularRowIndexEnd; ++RowIndex)
		{
			FJointSolverConstraintRowData& Constraint = RowDatas[RowIndex];
			switch (Constraint.UpdateType)
			{
			case EJointSolverConstraintUpdateType::Angular_Twist:
			case EJointSolverConstraintUpdateType::Angular_Cone:
			case EJointSolverConstraintUpdateType::Angular_SingleLimitedSwing:
				bNeedSwingTwist = true;
				break;
			case EJointSolverConstraintUpdateType::Angular_Locked:
				bNeedLockedAxes = true;
				break;
			}
		}
	}

	void FJointSolverConstraints::UpdateRotationConstraints(
		const TArray<FJointSolverConstraintRowData>& RowDatas,
		TArray<FJointSolverConstraintRowState>& RowStates,
		const FJointSolverJointState& JointState,
		const FPBDJointSettings& JointSettings)
	{
		const FRotation3 R0 = JointState.Rs[0];
		const FRotation3 R1 = JointState.Rs[1];
		const FRotation3 R01 = R0.Inverse() * R1;
		FRotation3 R01Twist = FRotation3::FromIdentity();
		FRotation3 R01Swing = FRotation3::FromIdentity();
		if (bNeedSwingTwist)
		{
			R01.ToSwingTwistX(R01Swing, R01Twist);
		}

		FVec3 Axes[3] = { FVec3(0), FVec3(0), FVec3(0) };
		if (bNeedLockedAxes)
		{
			FPBDJointUtilities::GetLockedRotationAxes(R0, R1, Axes[0], Axes[1], Axes[2]);
		}

		for (int32 RowIndex = AngularRowIndexBegin; RowIndex < AngularRowIndexEnd; ++RowIndex)
		{
			const FJointSolverConstraintRowData& RowData = RowDatas[RowIndex];
			FJointSolverConstraintRowState& RowState = RowStates[RowIndex];

			RowState.IterationReset();

			switch (RowData.UpdateType)
			{
			case EJointSolverConstraintUpdateType::Angular_Twist:
				check(bNeedSwingTwist);
				UpdateTwistConstraint(RowData, RowState, JointSettings, R1, R01Twist);
				break;
			case EJointSolverConstraintUpdateType::Angular_Cone:
				check(bNeedSwingTwist);
				UpdateConeSwingConstraint(RowData, RowState, JointSettings, R0, R01Swing);
				break;
			case EJointSolverConstraintUpdateType::Angular_SingleLockedSwing:
				UpdateSingleLockedSwingConstraint(RowData, RowState, JointSettings, R0, R1);
				break;
			case EJointSolverConstraintUpdateType::Angular_SingleLimitedSwing:
				check(bNeedSwingTwist);
				UpdateSingleLimitedSwingConstraint(RowData, RowState, JointSettings, R0, R01Swing);
				break;
			case EJointSolverConstraintUpdateType::Angular_DualConeSwing:
				UpdateDualConeSwingConstraint(RowData, RowState, JointSettings, R0, R1);
				break;
			case EJointSolverConstraintUpdateType::Angular_Locked:
			{
				check(bNeedLockedAxes);
				int32 AxisIndex = FJointConstants::AxisIndex((EJointAngularConstraintIndex)RowData.ConstraintIndex);
				RowState.Axis = Axes[AxisIndex];
				FReal Position = (AxisIndex == 0) ? R01.X : ((AxisIndex == 1) ? R01.Y : R01.Z); // Can't index Quat elements...
				RowState.CalculateError(Position, 0.0f);
				break;
			}
			case EJointSolverConstraintUpdateType::Angular_TwistDrive:
				UpdateTwistDrive(RowData, RowState, JointSettings, R1, R01Twist);
				break;
			case EJointSolverConstraintUpdateType::Angular_ConeDrive:
				UpdateConeSwingDrive(RowData, RowState, JointSettings, R0, R01Swing);
				break;
			case EJointSolverConstraintUpdateType::Angular_SwingDrive:
				UpdateSwingDrive(RowData, RowState, JointSettings, R0, R01Swing);
				break;
			case EJointSolverConstraintUpdateType::Angular_SLerpDrive:
				UpdateSLerpDrive(RowData, RowState, JointSettings, R0, R1);
				break;
			}
		}
	}

	//
	//////////////////////////////////////////////////////////////////////////
	//

	// TODO: Handle multi-dimensional constraints better
	void FJointSolverConstraints::AddPointPositionConstraint(
		TArray<FJointSolverConstraintRowData>& RowDatas,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		for (int32 AxisIndex = 0; AxisIndex < 3; ++AxisIndex)
		{
			int32 Index = RowDatas.AddDefaulted();
			LinearRowIndexEnd = Index + 1;

			FJointSolverConstraintRowData& RowData = RowDatas[Index];
			RowData.UpdateType = EJointSolverConstraintUpdateType::Linear_Point;
			RowData.NumRows = 1;
			RowData.Stiffness = FPBDJointUtilities::GetLinearStiffness(SolverSettings, JointSettings);
			RowData.JointIndex = JointIndex;
			RowData.ConstraintIndex = AxisIndex;
			RowData.Limit = 0.0f;
		}

		if (bChaos_Joint_MultiDimension)
		{
			RowDatas.Last(2).NumRows = 3;
			RowDatas.Last(1).NumRows = 0;
			RowDatas.Last(0).NumRows = 0;
		}
	}

	void FJointSolverConstraints::AddSphericalPositionConstraint(
		TArray<FJointSolverConstraintRowData>& RowDatas,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		int32 Index = RowDatas.AddDefaulted();
		LinearRowIndexEnd = Index + 1;

		// This is the soft version of 
		FJointSolverConstraintRowData& RowData = RowDatas[Index];
		RowData.UpdateType = EJointSolverConstraintUpdateType::Linear_Spherical;
		RowData.NumRows = 1;
		RowData.JointIndex = JointIndex;
		RowData.Limit = JointSettings.LinearLimit;
		if (JointSettings.bSoftLinearLimitsEnabled)
		{
			RowData.Stiffness = FPBDJointUtilities::GetSoftLinearStiffness(SolverSettings, JointSettings);
			RowData.Damping = FPBDJointUtilities::GetSoftLinearDamping(SolverSettings, JointSettings);
			RowData.bIsAccelerationMode = FPBDJointUtilities::GetLinearSoftAccelerationMode(SolverSettings, JointSettings);
			RowData.bIsSoft = true;
		}
		else
		{
			RowData.Stiffness = FPBDJointUtilities::GetLinearStiffness(SolverSettings, JointSettings);

		}
	}

	void FJointSolverConstraints::AddCylindricalPositionConstraint(
		TArray<FJointSolverConstraintRowData>& RowDatas,
		const int32 AxisIndex,
		const EJointMotionType AxialMotion,
		const EJointMotionType RadialMotion,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		{
			int32 AxialIndex = RowDatas.AddDefaulted(2);
			int32 RadialIndex = AxialIndex + 1;
			LinearRowIndexEnd = RadialIndex + 1;

			FJointSolverConstraintRowData& AxisRowData = RowDatas[AxialIndex];
			AxisRowData.UpdateType = EJointSolverConstraintUpdateType::Linear_Cylindrical;
			AxisRowData.NumRows = 1;
			AxisRowData.JointIndex = JointIndex;
			AxisRowData.ConstraintIndex = AxisIndex;
			if ((AxialMotion == EJointMotionType::Limited) && JointSettings.bSoftLinearLimitsEnabled)
			{
				AxisRowData.Stiffness = FPBDJointUtilities::GetSoftLinearStiffness(SolverSettings, JointSettings);
				AxisRowData.Damping = FPBDJointUtilities::GetSoftLinearDamping(SolverSettings, JointSettings);
				AxisRowData.bIsAccelerationMode = FPBDJointUtilities::GetLinearSoftAccelerationMode(SolverSettings, JointSettings);
				AxisRowData.bIsSoft = true;
				AxisRowData.Limit = JointSettings.LinearLimit;
			}
			else if (AxialMotion == EJointMotionType::Limited)
			{
				AxisRowData.Stiffness = FPBDJointUtilities::GetLinearStiffness(SolverSettings, JointSettings);
				AxisRowData.Limit = JointSettings.LinearLimit;
			}
			else if (AxialMotion == EJointMotionType::Locked)
			{
				AxisRowData.Stiffness = FPBDJointUtilities::GetLinearStiffness(SolverSettings, JointSettings);
				AxisRowData.Limit = 0.0f;
			}
			else
			{
				AxisRowData.Stiffness = 0.0f;
				AxisRowData.Limit = FLT_MAX;
			}

			FJointSolverConstraintRowData& RadialRowData = RowDatas[RadialIndex];
			RadialRowData.UpdateType = EJointSolverConstraintUpdateType::None;	// Handled by Cylindrical update on previous constraint
			RadialRowData.NumRows = 1;
			RadialRowData.JointIndex = JointIndex;
			RadialRowData.ConstraintIndex = AxisIndex;
			if ((RadialMotion == EJointMotionType::Limited) && JointSettings.bSoftLinearLimitsEnabled)
			{
				RadialRowData.Stiffness = FPBDJointUtilities::GetSoftLinearStiffness(SolverSettings, JointSettings);
				RadialRowData.Damping = FPBDJointUtilities::GetSoftLinearDamping(SolverSettings, JointSettings);
				RadialRowData.bIsAccelerationMode = FPBDJointUtilities::GetLinearSoftAccelerationMode(SolverSettings, JointSettings);
				RadialRowData.bIsSoft = true;
				RadialRowData.Limit = JointSettings.LinearLimit;
			}
			else if (RadialMotion == EJointMotionType::Limited)
			{
				RadialRowData.Stiffness = FPBDJointUtilities::GetLinearStiffness(SolverSettings, JointSettings);
				RadialRowData.Limit = JointSettings.LinearLimit;
			}
			else if (RadialMotion == EJointMotionType::Locked)
			{
				RadialRowData.Stiffness = FPBDJointUtilities::GetLinearStiffness(SolverSettings, JointSettings);
				RadialRowData.Limit = 0.0f;
			}
			else
			{
				RadialRowData.Stiffness = 0.0f;
				RadialRowData.Limit = FLT_MAX;
			}
		}
	}

	void FJointSolverConstraints::AddPlanarPositionConstraint(
		TArray<FJointSolverConstraintRowData>& RowDatas,
		const int32 AxisIndex,
		const EJointMotionType AxialMotion,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		int32 Index = RowDatas.AddDefaulted();
		LinearRowIndexEnd = Index + 1;

		FJointSolverConstraintRowData& RowData = RowDatas[Index];
		RowData.UpdateType = EJointSolverConstraintUpdateType::Linear_Planar;
		RowData.NumRows = 1;
		RowData.JointIndex = JointIndex;
		RowData.Limit = JointSettings.LinearLimit;

		if ((AxialMotion == EJointMotionType::Limited) && JointSettings.bSoftLinearLimitsEnabled)
		{
			RowData.Stiffness = FPBDJointUtilities::GetSoftLinearStiffness(SolverSettings, JointSettings);
			RowData.Damping = FPBDJointUtilities::GetSoftLinearDamping(SolverSettings, JointSettings);
			RowData.bIsAccelerationMode = FPBDJointUtilities::GetLinearSoftAccelerationMode(SolverSettings, JointSettings);
			RowData.bIsSoft = true;
		}
		else
		{
			RowData.Stiffness = FPBDJointUtilities::GetLinearStiffness(SolverSettings, JointSettings);
		}
	}

	void FJointSolverConstraints::AddPositionDrive(
		TArray<FJointSolverConstraintRowData>& RowDatas,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings,
		const EJointSolverConstraintUpdateType UpdateType,
		const int32 AxisIndex)
	{
		int32 Index = RowDatas.AddDefaulted();
		LinearRowIndexEnd = Index + 1;

		FJointSolverConstraintRowData& RowData = RowDatas[Index];
		RowData.UpdateType = UpdateType;
		RowData.NumRows = 1;
		RowData.JointIndex = JointIndex;
		RowData.ConstraintIndex = AxisIndex;
		RowData.Limit = 0.0f;
		RowData.Stiffness = FPBDJointUtilities::GetLinearDriveStiffness(SolverSettings, JointSettings, AxisIndex);
		RowData.Damping = FPBDJointUtilities::GetLinearDriveDamping(SolverSettings, JointSettings, AxisIndex);
		RowData.bIsAccelerationMode = FPBDJointUtilities::GetLinearDriveAccelerationMode(SolverSettings, JointSettings);
		RowData.bIsSoft = true;
	}

	//
	//////////////////////////////////////////////////////////////////////////
	//


	void FJointSolverConstraints::AddTwistConstraint(
		TArray<FJointSolverConstraintRowData>& RowDatas,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings,
		const EJointSolverConstraintUpdateType Type,
		const bool bUseSoftLimit)
	{
		int32 Index = RowDatas.AddDefaulted();
		FJointSolverConstraintRowData& RowData = RowDatas[Index];
		AngularRowIndexEnd = Index + 1;

		RowData.JointIndex = JointIndex;
		RowData.UpdateType = Type;
		RowData.ConstraintIndex = (int32)EJointAngularConstraintIndex::Twist;
		RowData.NumRows = 1;
		RowData.Limit = JointSettings.AngularLimits[(int32)EJointAngularConstraintIndex::Twist];

		if (bUseSoftLimit)
		{
			RowData.Stiffness = FPBDJointUtilities::GetSoftTwistStiffness(SolverSettings, JointSettings);
			RowData.Damping = FPBDJointUtilities::GetSoftTwistDamping(SolverSettings, JointSettings);
			RowData.bIsAccelerationMode = FPBDJointUtilities::GetAngularSoftAccelerationMode(SolverSettings, JointSettings);
			RowData.bIsSoft = true;
		}
		else
		{
			RowData.Stiffness = FPBDJointUtilities::GetTwistStiffness(SolverSettings, JointSettings);
		}
	}

	void FJointSolverConstraints::AddSwingConstraint(
		TArray<FJointSolverConstraintRowData>& RowDatas,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings,
		const EJointSolverConstraintUpdateType Type,
		const EJointAngularConstraintIndex ConstraintIndex,
		const bool bUseSoftLimit)
	{
		int32 Index = RowDatas.AddDefaulted();
		FJointSolverConstraintRowData& RowData = RowDatas[Index];
		AngularRowIndexEnd = Index + 1;

		RowData.JointIndex = JointIndex;
		RowData.UpdateType = Type;
		RowData.ConstraintIndex = (int32)ConstraintIndex;
		RowData.NumRows = 1;
		RowData.Limit = JointSettings.AngularLimits[(int32)ConstraintIndex];

		if (bUseSoftLimit)
		{
			RowData.Stiffness = FPBDJointUtilities::GetSoftSwingStiffness(SolverSettings, JointSettings);
			RowData.Damping = FPBDJointUtilities::GetSoftSwingDamping(SolverSettings, JointSettings);
			RowData.bIsAccelerationMode = FPBDJointUtilities::GetAngularSoftAccelerationMode(SolverSettings, JointSettings);
			RowData.bIsSoft = true;
		}
		else
		{
			RowData.Stiffness = FPBDJointUtilities::GetSwingStiffness(SolverSettings, JointSettings);
		}
	}

	void FJointSolverConstraints::AddTwistDrive(
		TArray<FJointSolverConstraintRowData>& RowDatas,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		int32 Index = RowDatas.AddDefaulted();
		FJointSolverConstraintRowData& RowData = RowDatas[Index];
		AngularRowIndexEnd = Index + 1;

		RowData.JointIndex = JointIndex;
		RowData.UpdateType = EJointSolverConstraintUpdateType::Angular_TwistDrive;
		RowData.NumRows = 1;
		RowData.Limit = 0.0f;

		RowData.Stiffness = FPBDJointUtilities::GetAngularTwistDriveStiffness(SolverSettings, JointSettings);
		RowData.Damping = FPBDJointUtilities::GetAngularTwistDriveDamping(SolverSettings, JointSettings);
		RowData.bIsAccelerationMode = FPBDJointUtilities::GetAngularDriveAccelerationMode(SolverSettings, JointSettings);
		RowData.bIsSoft = true;
	}

	void FJointSolverConstraints::AddSwingDrive(
		TArray<FJointSolverConstraintRowData>& RowDatas,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings,
		const EJointSolverConstraintUpdateType Type,
		const EJointAngularConstraintIndex ConstraintIndex)
	{
		int32 Index = RowDatas.AddDefaulted();
		FJointSolverConstraintRowData& RowData = RowDatas[Index];
		AngularRowIndexEnd = Index + 1;

		RowData.JointIndex = JointIndex;
		RowData.UpdateType = Type;
		RowData.ConstraintIndex = (int32)ConstraintIndex;
		RowData.NumRows = 1;
		RowData.Limit = 0.0f;

		RowData.Stiffness = FPBDJointUtilities::GetAngularSwingDriveStiffness(SolverSettings, JointSettings);
		RowData.Damping = FPBDJointUtilities::GetAngularSwingDriveDamping(SolverSettings, JointSettings);
		RowData.bIsAccelerationMode = FPBDJointUtilities::GetAngularDriveAccelerationMode(SolverSettings, JointSettings);
		RowData.bIsSoft = true;
	}

	void FJointSolverConstraints::AddSLerpDrive(
		TArray<FJointSolverConstraintRowData>& RowDatas,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		int32 Index = RowDatas.AddDefaulted();
		FJointSolverConstraintRowData& RowData = RowDatas[Index];
		AngularRowIndexEnd = Index + 1;

		RowData.JointIndex = JointIndex;
		RowData.UpdateType = EJointSolverConstraintUpdateType::Angular_SLerpDrive;
		RowData.NumRows = 1;
		RowData.Limit = 0.0f;

		RowData.Stiffness = FPBDJointUtilities::GetAngularSLerpDriveStiffness(SolverSettings, JointSettings);
		RowData.Damping = FPBDJointUtilities::GetAngularSLerpDriveDamping(SolverSettings, JointSettings);
		RowData.bIsAccelerationMode = FPBDJointUtilities::GetAngularDriveAccelerationMode(SolverSettings, JointSettings);
		RowData.bIsSoft = true;
	}

	//
	//////////////////////////////////////////////////////////////////////////
	//

	int32 FJointSolver::ApplyPositionConstraints(
		const FReal Dt,
		TArray<FJointSolverJointState>& JointStates,
		const TArray <FJointSolverConstraintRowData>& RowDatas,
		TArray <FJointSolverConstraintRowState>& RowStates,
		int32 JointIndexBegin,
		int32 JointIndexEnd,
		int32 RowIndexBegin,
		int32 RowIndexEnd)
	{
		int32 NumActiveConstraints = 0;

		// Calculate the delta position and rotation to fix the constraint errors
		// NOTE: Rows are in Joint order, so we can loop over them directly rather than looping over joints and then their rows.
		for (int32 RowIndex = RowIndexBegin; RowIndex < RowIndexEnd; /* no op */)
		{
			const int32 RowJointIndex = RowDatas[RowIndex].JointIndex;
			if (RowDatas[RowIndex].NumRows == 3)
			{
				FVec3 Error = FVec3(RowStates[RowIndex].Error, RowStates[RowIndex + 1].Error, RowStates[RowIndex + 2].Error);
				if (Error.Size() > JointStates[RowJointIndex].PositionTolerance)
				{
					ApplyPositionConstraint3(Dt, JointStates[RowJointIndex], RowDatas[RowIndex], RowStates[RowIndex], RowStates[RowIndex + 1], RowStates[RowIndex + 2]);
					JointStates[RowJointIndex].DPs[0] += RowStates[RowIndex].DPs[0];
					JointStates[RowJointIndex].DPs[1] += RowStates[RowIndex].DPs[1];
					JointStates[RowJointIndex].DRs[0] += RowStates[RowIndex].DRs[0];
					JointStates[RowJointIndex].DRs[1] += RowStates[RowIndex].DRs[1];
					NumActiveConstraints += 3;
				}
				RowIndex += 3;
			}
			else if (RowDatas[RowIndex].NumRows == 1)
			{
				if (FMath::Abs(RowStates[RowIndex].Error) > JointStates[RowJointIndex].PositionTolerance)
				{
					ApplyPositionConstraint1(Dt, JointStates[RowJointIndex], RowDatas[RowIndex], RowStates[RowIndex]);
					JointStates[RowJointIndex].DPs[0] += RowStates[RowIndex].DPs[0];
					JointStates[RowJointIndex].DPs[1] += RowStates[RowIndex].DPs[1];
					JointStates[RowJointIndex].DRs[0] += RowStates[RowIndex].DRs[0];
					JointStates[RowJointIndex].DRs[1] += RowStates[RowIndex].DRs[1];
					++NumActiveConstraints;
				}
				++RowIndex;
			}
		}

		// Accumulate and apply the deltas for each joint
		for (int32 JointIndex = JointIndexBegin; JointIndex < JointIndexEnd; ++JointIndex)
		{
			JointStates[JointIndex].ApplyDelta(JointStates[JointIndex].DPs[0], JointStates[JointIndex].DRs[0], JointStates[JointIndex].DPs[1], JointStates[JointIndex].DRs[1]);
		}

		return NumActiveConstraints;
	}


	int32 FJointSolver::ApplyRotationConstraints(
		const FReal Dt,
		TArray<FJointSolverJointState>& JointStates,
		const TArray <FJointSolverConstraintRowData>& RowDatas,
		TArray <FJointSolverConstraintRowState>& RowStates,
		int32 JointIndexBegin,
		int32 JointIndexEnd,
		int32 RowIndexBegin,
		int32 RowIndexEnd)
	{
		int32 NumActiveConstraints = 0;

		// Calculate the delta position and rotation to fix the constraint errors
		// NOTE: Rows are in Joint order, so we can loop over them directly rather than looping over joints and then their rows.
		for (int32 RowIndex = RowIndexBegin; RowIndex < RowIndexEnd; ++RowIndex)
		{
			const int32 RowJointIndex = RowDatas[RowIndex].JointIndex;

			if (FMath::Abs(RowStates[RowIndex].Error) > JointStates[RowJointIndex].AngleTolerance)
			{
				ApplyRotationConstraint(Dt, JointStates[RowJointIndex], RowDatas[RowIndex], RowStates[RowIndex]);

				JointStates[RowJointIndex].DRs[0] += RowStates[RowIndex].DRs[0];
				JointStates[RowJointIndex].DRs[1] += RowStates[RowIndex].DRs[1];
				++NumActiveConstraints;
			}
		}

		// Accumulate and apply the deltas for each joint
		for (int32 JointIndex = JointIndexBegin; JointIndex < JointIndexEnd; ++JointIndex)
		{
			JointStates[JointIndex].ApplyRotationDelta(JointStates[JointIndex].DRs[0], JointStates[JointIndex].DRs[1]);
		}

		return NumActiveConstraints;
	}
}