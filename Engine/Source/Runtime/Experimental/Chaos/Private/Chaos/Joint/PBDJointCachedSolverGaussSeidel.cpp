// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/Joint/PBDJointCachedSolverGaussSeidel.h"
#include "Chaos/Joint/ChaosJointLog.h"
#include "Chaos/Joint/JointConstraintsCVars.h"
#include "Chaos/Joint/JointSolverConstraints.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/PBDJointConstraintUtilities.h"
#include "Chaos/Utilities.h"
#include "Chaos/VectorUtility.h"


namespace Chaos
{

/** Derived states management */

void FPBDJointCachedSolver::InitDerivedState()
{
	InitConnectorXs[0] = X(0) + R(0) * LocalConnectorXs[0].GetTranslation();
	InitConnectorXs[1] = X(1) + R(1) * LocalConnectorXs[1].GetTranslation();
	InitConnectorRs[0] = R(0) * LocalConnectorXs[0].GetRotation();
	InitConnectorRs[1] = R(1) * LocalConnectorXs[1].GetRotation();
	InitConnectorRs[1].EnforceShortestArcWith(InitConnectorRs[0]);

	ComputeBodyState(0);
	ComputeBodyState(1);

	ConnectorRs[1].EnforceShortestArcWith(ConnectorRs[0]);

	ConnectorWDts[0] = FRotation3::CalculateAngularVelocity(InitConnectorRs[0], ConnectorRs[0], 1.0f);
	ConnectorWDts[1] = FRotation3::CalculateAngularVelocity(InitConnectorRs[1], ConnectorRs[1], 1.0f);

	ConnectorWDtsSimd[0] = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(ConnectorWDts[0][0], ConnectorWDts[0][1], ConnectorWDts[0][2], 0.0f));
	ConnectorWDtsSimd[1] = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(ConnectorWDts[1][0], ConnectorWDts[1][1], ConnectorWDts[1][2], 0.0f));
}

void FPBDJointCachedSolver::ComputeBodyState(const int32 BodyIndex)
{
	CurrentPs[BodyIndex] = P(BodyIndex);
	CurrentQs[BodyIndex] = Q(BodyIndex);
	ConnectorXs[BodyIndex] = CurrentPs[BodyIndex] + CurrentQs[BodyIndex] * LocalConnectorXs[BodyIndex].GetTranslation();
	ConnectorRs[BodyIndex] = CurrentQs[BodyIndex] * LocalConnectorXs[BodyIndex].GetRotation();
}

void FPBDJointCachedSolver::UpdateDerivedState()
{
	// Kinematic bodies will not be moved, so we don't update derived state during iterations
	if (InvM(0) > UE_SMALL_NUMBER)
	{
		ComputeBodyState(0);
	}
	if (InvM(1) > UE_SMALL_NUMBER)
	{
		ComputeBodyState(1);
	}
	ConnectorRs[1].EnforceShortestArcWith(ConnectorRs[0]);
}

void FPBDJointCachedSolver::UpdateDerivedState(const int32 BodyIndex)
{
	ComputeBodyState(BodyIndex);
	ConnectorRs[1].EnforceShortestArcWith(ConnectorRs[0]);
}

bool FPBDJointCachedSolver::UpdateIsActive()
{
	// NumActiveConstraints is initialized to -1, so there's no danger of getting invalid LastPs/Qs
	// We also check SolverStiffness mainly for testing when solver stiffness is 0 (so we don't exit immediately)
	if ((NumActiveConstraints >= 0) && (SolverStiffness > 0.0f))
	{
		bool bIsSolved =
			FVec3::IsNearlyEqual(Body(0).DP(), LastDPs[0], PositionTolerance)
			&& FVec3::IsNearlyEqual(Body(1).DP(), LastDPs[1], PositionTolerance)
			&& FVec3::IsNearlyEqual(Body(0).DQ(), LastDQs[0], 0.5f * AngleTolerance)
			&& FVec3::IsNearlyEqual(Body(1).DQ(), LastDQs[1], 0.5f * AngleTolerance);
		bIsActive = !bIsSolved;
	}

	LastDPs[0] = Body(0).DP();
	LastDPs[1] = Body(1).DP();
	LastDQs[0] = Body(0).DQ();
	LastDQs[1] = Body(1).DQ();

	return bIsActive;
}

void FPBDJointCachedSolver::Update(
	const FReal Dt,
	const FPBDJointSolverSettings& SolverSettings,
	const FPBDJointSettings& JointSettings)
{
	//UpdateIsActive();
}

void FPBDJointCachedSolver::UpdateMass0(const FReal& InInvM, const FVec3& InInvIL)
{
	if (Body0().IsDynamic())
	{
		InvMs[0] = InInvM;
		InvIs[0] = Utilities::ComputeWorldSpaceInertia(CurrentQs[0], InInvIL);
	}
	else
	{
		InvMs[0] = 0;
		InvIs[0] = FMatrix33(0);
	}
}

void FPBDJointCachedSolver::UpdateMass1(const FReal& InInvM, const FVec3& InInvIL)
{
	if (Body1().IsDynamic())
	{
		InvMs[1] = InInvM;
		InvIs[1] = Utilities::ComputeWorldSpaceInertia(CurrentQs[1], InInvIL);
	}
	else
	{
		InvMs[1] = 0;
		InvIs[1] = FMatrix33(0);
	}
}

void FPBDJointCachedSolver::SetShockPropagationScales(const FReal InvMScale0, const FReal InvMScale1, const FReal Dt)
{
	bool bNeedsUpdate = false;
	if (Body0().ShockPropagationScale() != InvMScale0 && Body0().ShockPropagationScale() > 0.0f)
	{
		const FReal Mult = InvMScale0 / Body0().ShockPropagationScale();
		InvMs[0] *= Mult;
		InvIs[0] *= Mult;
		Body0().SetShockPropagationScale(InvMScale0);
		bNeedsUpdate = true;
	}
	if (Body1().ShockPropagationScale() != InvMScale1 && Body1().ShockPropagationScale() > 0.0f)
	{
		const FReal Mult = InvMScale1 / Body1().ShockPropagationScale();
		InvMs[1] *= Mult;
		InvIs[1] *= Mult;
		Body1().SetShockPropagationScale(InvMScale1);
		bNeedsUpdate = true;
	}
	if (bNeedsUpdate)
	{
		for (int32 ConstraintIndex = 0; ConstraintIndex < 3; ++ConstraintIndex)
		{
			if (PositionConstraints.GetValidDatas(ConstraintIndex))
			{
				InitPositionDatasMass(PositionConstraints, ConstraintIndex, Dt);
			}
			if (RotationConstraints.GetValidDatas(ConstraintIndex))
			{
				InitRotationDatasMass(RotationConstraints, ConstraintIndex, Dt);
			}
			if (PositionDrives.GetValidDatas(ConstraintIndex))
			{
				InitPositionDatasMass(PositionDrives, ConstraintIndex, Dt);
			}
			if (RotationDrives.GetValidDatas(ConstraintIndex))
			{
				InitRotationDatasMass(RotationDrives, ConstraintIndex, Dt);
			}
		}
	}
}

/** Main init function to cache datas that could be reused in the apply */

void FPBDJointCachedSolver::Init(
	const FReal Dt,
	const FPBDJointSolverSettings& SolverSettings,
	const FPBDJointSettings& JointSettings,
	const FRigidTransform3& XL0,
	const FRigidTransform3& XL1)
{
	LocalConnectorXs[0] = XL0;
	LocalConnectorXs[1] = XL1;

	// \todo(chaos): joint should support parent/child in either order
	SolverBodies[0].SetInvMScale(JointSettings.ParentInvMassScale);
	SolverBodies[1].SetInvMScale(FReal(1));
	SolverBodies[0].SetInvIScale(JointSettings.ParentInvMassScale);
	SolverBodies[1].SetInvIScale(FReal(1));
	SolverBodies[0].SetShockPropagationScale(FReal(1));
	SolverBodies[1].SetShockPropagationScale(FReal(1));

	// Tolerances are positional errors below visible detection. But in PBD the errors
	// we leave behind get converted to velocity, so we need to ensure that the resultant
	// movement from that erroneous velocity is less than the desired position tolerance.
	// Assume that the tolerances were defined for a 60Hz simulation, then it must be that
	// the position error is less than the position change from constant external forces
	// (e.g., gravity). So, we are saying that the tolerance was chosen because the position
	// error is less that F.dt^2. We need to scale the tolerance to work at our current dt.
	const FReal ToleranceScale = FMath::Min(1.f, 60.f * 60.f * Dt * Dt);
	PositionTolerance = ToleranceScale * SolverSettings.PositionTolerance;
	AngleTolerance = ToleranceScale * SolverSettings.AngleTolerance;

	NumActiveConstraints = -1;
	bIsActive = true;
	bIsBroken = false;
	bIsViolating = false;
	bUseSimd = SolverSettings.bUseSimd;
	bUsePositionBasedDrives = SolverSettings.bUsePositionBasedDrives;

	SolverStiffness = 1.0f;

	InitDerivedState();

	// Set the mass and inertia.
	// If enabled, adjust the mass so that we limit the maximum mass and inertia ratios
	FReal ConditionedInvMs[2] =
	{
		Body0().InvM(),
		Body1().InvM()
	};
	FVec3 ConditionedInvILs[2] =
	{
		Body0().InvILocal(),
		Body1().InvILocal()
	};
	if (JointSettings.bMassConditioningEnabled)
	{
		FPBDJointUtilities::ConditionInverseMassAndInertia(Body0().InvM(), Body1().InvM(), Body0().InvILocal(), Body1().InvILocal(), SolverSettings.MinParentMassRatio, SolverSettings.MaxInertiaRatio, ConditionedInvMs[0], ConditionedInvMs[1], ConditionedInvILs[0], ConditionedInvILs[1]);
	}
	UpdateMass0(ConditionedInvMs[0], ConditionedInvILs[0]);
	UpdateMass1(ConditionedInvMs[1], ConditionedInvILs[1]);

	// Cache all the informations for the position and rotation constraints
	const bool bResetLambdas = true;	// zero accumulators on full init
	InitPositionConstraints(Dt, SolverSettings, JointSettings, bResetLambdas);
	InitRotationConstraints(Dt, SolverSettings, JointSettings, bResetLambdas);

	InitPositionDrives(Dt, SolverSettings, JointSettings);
	InitRotationDrives(Dt, SolverSettings, JointSettings);

	LastDPs[0] = FVec3(0.f);
	LastDPs[1] = FVec3(0.f);
	LastDQs[0] = FVec3(0.f);
	LastDQs[1] = FVec3(0.f);
}

void FPBDJointCachedSolver::InitProjection(
	const FReal Dt,
	const FPBDJointSolverSettings& SolverSettings,
	const FPBDJointSettings& JointSettings)
{
	const FReal LinearProjection = FPBDJointUtilities::GetLinearProjection(SolverSettings, JointSettings);
	const FReal AngularProjection = FPBDJointUtilities::GetAngularProjection(SolverSettings, JointSettings);
	const bool bHasLinearProjection = JointSettings.bProjectionEnabled && ((LinearProjection > 0) || (JointSettings.TeleportDistance > 0));
	// Teleport angle is not implemented in this linear solver, not need to initialize rotation for teleport. 
	const bool bHasAngularProjection = JointSettings.bProjectionEnabled && ((AngularProjection > 0) /*|| (JointSettings.TeleportAngle > 0)*/);

	if (bHasLinearProjection || bHasAngularProjection)
	{
		ComputeBodyState(0);
		ComputeBodyState(1);

		ConnectorRs[1].EnforceShortestArcWith(ConnectorRs[0]);

		// Fake spherical inertia for angular projection (avoid cost of recomputing inertia at current world space rotation)
		InvMs[0] = 0;
		InvIs[0] = FMatrix33(0, 0, 0);
		InvMs[1] = Body1().InvM();
		InvIs[1] = FMatrix33::FromDiagonal(FVec3(Body1().InvILocal().GetMin()));

		// We are reusing the constraints for projection but we don't want to reset the accumulated corrections
		const bool bResetLambdas = false;

		if (bHasLinearProjection)
		{
			InitPositionConstraints(Dt, SolverSettings, JointSettings, bResetLambdas);
		}

		if (bHasAngularProjection)
		{
			InitRotationConstraints(Dt, SolverSettings, JointSettings, bResetLambdas);
		}
	}
}

void FPBDJointCachedSolver::Deinit()
{
	SolverBodies[0].Reset();
	SolverBodies[1].Reset();
}

/** Main Apply function to solve all the constraint*/

void FPBDJointCachedSolver::ApplyConstraints(
	const FReal Dt,
	const FReal InSolverStiffness,
	const FPBDJointSolverSettings& SolverSettings,
	const FPBDJointSettings& JointSettings)
{
	NumActiveConstraints = 0;
	SolverStiffness = InSolverStiffness;

	if (SolverSettings.bSolvePositionLast)
	{
		ApplyRotationConstraints(Dt);
		ApplyPositionConstraints(Dt);

		ApplyRotationDrives(Dt);
		ApplyPositionDrives(Dt);
	}
	else
	{
		ApplyPositionConstraints(Dt);
		ApplyRotationConstraints(Dt);

		ApplyPositionDrives(Dt);
		ApplyRotationDrives(Dt);
	}
}

void FPBDJointCachedSolver::ApplyVelocityConstraints(
	const FReal Dt,
	const FReal InSolverStiffness,
	const FPBDJointSolverSettings& SolverSettings,
	const FPBDJointSettings& JointSettings)
{
	SolverStiffness = InSolverStiffness;

	// This is used for the QuasiPbd solver. If the Pbd step applied impulses to
	// correct position errors, it will have introduced a velocity equal to the 
	// correction divided by the timestep. We ensure that the velocity constraints
	// (including restitution) are also enforced. This also prevents any position
	// errors from the previous frame getting converted into energy.

	if (SolverSettings.bSolvePositionLast)
	{
		ApplyAngularVelocityConstraints();
		ApplyLinearVelocityConstraints();

		ApplyRotationVelocityDrives(Dt);
		ApplyPositionVelocityDrives(Dt);
	}
	else
	{
		ApplyLinearVelocityConstraints();
		ApplyAngularVelocityConstraints();

		ApplyPositionVelocityDrives(Dt);
		ApplyRotationVelocityDrives(Dt);
	}
}

/** UTILS FOR POSITION CONSTRAINT **************************************************************************************/

FORCEINLINE bool ExtractLinearMotion(const FPBDJointSettings& JointSettings,
	TVec3<bool>& bLinearLocked, TVec3<bool>& bLinearLimited)
{
	bool bHasPositionConstraints =
		(JointSettings.LinearMotionTypes[0] != EJointMotionType::Free)
		|| (JointSettings.LinearMotionTypes[1] != EJointMotionType::Free)
		|| (JointSettings.LinearMotionTypes[2] != EJointMotionType::Free);
	if (!bHasPositionConstraints)
	{
		return false;
	}

	const TVec3<EJointMotionType>& LinearMotion = JointSettings.LinearMotionTypes;
	bLinearLocked =
	{
		(LinearMotion[0] == EJointMotionType::Locked),
		(LinearMotion[1] == EJointMotionType::Locked),
		(LinearMotion[2] == EJointMotionType::Locked),
	};
	bLinearLimited =
	{
		(LinearMotion[0] == EJointMotionType::Limited),
		(LinearMotion[1] == EJointMotionType::Limited),
		(LinearMotion[2] == EJointMotionType::Limited),
	};
	return true;
}

/** INIT POSITION CONSTRAINT ******************************************************************************************/

void FPBDJointCachedSolver::InitPositionConstraints(
	const FReal Dt,
	const FPBDJointSolverSettings& SolverSettings,
	const FPBDJointSettings& JointSettings,
	const bool bResetLambdas)
{
	PositionConstraints.SetValidDatas(0, false);
	PositionConstraints.SetValidDatas(1, false);
	PositionConstraints.SetValidDatas(2, false);
	PositionConstraints.bUseSimd = false;

	TVec3<bool> bLinearLocked, bLinearLimited;
	if (!ExtractLinearMotion(JointSettings, bLinearLocked, bLinearLimited))
		return;

	PositionConstraints.bUseSimd = bUseSimd && bLinearLocked[0] && bLinearLocked[1] && bLinearLocked[2];
	PositionConstraints.bAccelerationMode = FPBDJointUtilities::GetLinearSoftAccelerationMode(SolverSettings, JointSettings);

	if (PositionConstraints.bUseSimd)
	{
		FRealSingle HardStiffness = FRealSingle(FPBDJointUtilities::GetLinearStiffness(SolverSettings, JointSettings));
		PositionConstraints.Simd.ConstraintHardStiffness = VectorLoadFloat1(&HardStiffness);

		if (bResetLambdas)
		{
			PositionConstraints.Simd.ConstraintLambda = VectorZeroFloat();
		}
	}
	else
	{
		for (int32 ConstraintIndex = 0; ConstraintIndex < 3; ++ConstraintIndex)
		{
			PositionConstraints.InitDatas(ConstraintIndex, bLinearLimited[ConstraintIndex] &&
				FPBDJointUtilities::GetSoftLinearLimitEnabled(SolverSettings, JointSettings),
				FPBDJointUtilities::GetSoftLinearStiffness(SolverSettings, JointSettings),
				FPBDJointUtilities::GetSoftLinearDamping(SolverSettings, JointSettings),
				FPBDJointUtilities::GetLinearStiffness(SolverSettings, JointSettings),
				bResetLambdas);
		}
	}

	const TVec3<EJointMotionType>& LinearMotion = JointSettings.LinearMotionTypes;

	if (bLinearLocked[0] || bLinearLocked[1] || bLinearLocked[2])
	{
		if (PositionConstraints.bUseSimd)
		{
			InitLockedPositionConstraintSimd(JointSettings, Dt, LinearMotion);
		}
		else
		{
			// Process locked constraints
			InitLockedPositionConstraint(JointSettings, Dt, LinearMotion);
		}
	}
	if (bLinearLimited[0] || bLinearLimited[1] || bLinearLimited[2])
	{
		check(PositionConstraints.bUseSimd == false);
		// Process limited constraints
		if (bLinearLimited[0] && bLinearLimited[1] && bLinearLimited[2])
		{
			// Spherical constraint
			InitSphericalPositionConstraint(JointSettings, Dt);
		}
		else if (bLinearLimited[1] && bLinearLimited[2])
		{
			// Cylindrical constraint along X axis
			InitCylindricalPositionConstraint(JointSettings, Dt, 0);
		}
		else if (bLinearLimited[0] && bLinearLimited[2])
		{
			// Cylindrical constraint along Y axis
			InitCylindricalPositionConstraint(JointSettings, Dt, 1);
		}
		else if (bLinearLimited[0] && bLinearLimited[1])
		{
			// Cylindrical constraint along Z axis
			InitCylindricalPositionConstraint(JointSettings, Dt, 2);
		}
		else if (bLinearLimited[0])
		{
			// Planar constraint along X axis
			InitPlanarPositionConstraint(JointSettings, Dt, 0);
		}
		else if (bLinearLimited[1])
		{
			// Planar constraint along Y axis
			InitPlanarPositionConstraint(JointSettings, Dt, 1);
		}
		else if (bLinearLimited[2])
		{
			// Planar constraint along Z axis
			InitPlanarPositionConstraint(JointSettings, Dt, 2);
		}
	}
}

void FPBDJointCachedSolver::InitPositionDatasMass(
	FAxisConstraintDatas& PositionDatas,
	const int32 ConstraintIndex,
	const FReal Dt)
{
	const FVec3 AngularAxis0 = FVec3::CrossProduct(PositionDatas.Data.ConstraintArms[ConstraintIndex][0], PositionDatas.Data.ConstraintAxis[ConstraintIndex]);
	const FVec3 AngularAxis1 = FVec3::CrossProduct(PositionDatas.Data.ConstraintArms[ConstraintIndex][1], PositionDatas.Data.ConstraintAxis[ConstraintIndex]);
	const FVec3 IA0 = Utilities::Multiply(InvI(0), AngularAxis0);
	const FVec3 IA1 = Utilities::Multiply(InvI(1), AngularAxis1);
	const FReal II0 = FVec3::DotProduct(AngularAxis0, IA0);
	const FReal II1 = FVec3::DotProduct(AngularAxis1, IA1);

	PositionDatas.UpdateMass(ConstraintIndex, IA0, IA1, InvM(0) + II0 + InvM(1) + II1, Dt, bUsePositionBasedDrives);
}

void FPBDJointCachedSolver::SetInitConstraintVelocity(
	const FVec3& ConstraintArm0,
	const FVec3& ConstraintArm1)

{
	const FVec3 CV0 = V(0) + FVec3::CrossProduct(W(0), ConstraintArm0);
	const FVec3 CV1 = V(1) + FVec3::CrossProduct(W(1), ConstraintArm1);
	const FVec3 CV = CV1 - CV0;
	InitConstraintVelocity = CV;
}

void FPBDJointCachedSolver::InitPositionConstraintDatas(
	const int32 ConstraintIndex,
	const FVec3& ConstraintAxis,
	const FReal& ConstraintDelta,
	const FReal ConstraintRestitution,
	const FReal Dt,
	const FReal ConstraintLimit,
	const EJointMotionType JointType,
	const FVec3& ConstraintArm0,
	const FVec3& ConstraintArm1)
{
	const FVec3 LocalAxis = (ConstraintDelta < 0.0f) ? -ConstraintAxis : ConstraintAxis;
	const FReal LocalDelta = (ConstraintDelta < 0.0f) ? -ConstraintDelta : ConstraintDelta;

	PositionConstraints.SetMotionType(ConstraintIndex, JointType);

	if (JointType == EJointMotionType::Locked)
	{
		PositionConstraints.Data.ConstraintLimits[ConstraintIndex] = 0.0f;
		PositionConstraints.UpdateDatas(ConstraintIndex, LocalAxis, LocalDelta,
			0.0, false, ConstraintArm0, ConstraintArm1);
	}
	else if (JointType == EJointMotionType::Limited)
	{
		PositionConstraints.Data.ConstraintLimits[ConstraintIndex] = ConstraintLimit;
		PositionConstraints.UpdateDatas(ConstraintIndex, LocalAxis, LocalDelta,
			ConstraintRestitution, true, ConstraintArm0, ConstraintArm1);
	}

	InitConstraintAxisLinearVelocities[ConstraintIndex] = FVec3::DotProduct(InitConstraintVelocity, LocalAxis);

	InitPositionDatasMass(PositionConstraints, ConstraintIndex, Dt);
}


void FPBDJointCachedSolver::InitLockedPositionConstraint(
	const FPBDJointSettings& JointSettings,
	const FReal Dt,
	const TVec3<EJointMotionType>& LinearMotion)
{
	FVec3 ConstraintArm0 = ConnectorXs[1] - CurrentPs[0];
	const FVec3 ConstraintArm1 = ConnectorXs[1] - CurrentPs[1];

	const FVec3 DX = ConnectorXs[1] - ConnectorXs[0];
	const FMatrix33 R0M = ConnectorRs[0].ToMatrix();
	FVec3 CX = FVec3::ZeroVector;

	// For a locked constraint we try to match an exact constraint, 
	// it is why we are adding back the constraint projection along each axis.
	// If the 3 axis are locked the constraintarm0 is then ConnectorXs[0] - CurrentPs[0];
	for (int32 ConstraintIndex = 0; ConstraintIndex < 3; ++ConstraintIndex)
	{
		if (LinearMotion[ConstraintIndex] == EJointMotionType::Locked)
		{
			const FVec3 ConstraintAxis = R0M.GetAxis(ConstraintIndex);
			CX[ConstraintIndex] = FVec3::DotProduct(DX, ConstraintAxis);
			ConstraintArm0 -= ConstraintAxis * CX[ConstraintIndex];

		}
	}

	SetInitConstraintVelocity(ConstraintArm0, ConstraintArm1);

	for (int32 ConstraintIndex = 0; ConstraintIndex < 3; ++ConstraintIndex)
	{
		if (LinearMotion[ConstraintIndex] == EJointMotionType::Locked)
		{
			const FVec3 ConstraintAxis = R0M.GetAxis(ConstraintIndex);
			InitPositionConstraintDatas(ConstraintIndex, ConstraintAxis, CX[ConstraintIndex], 0.0, Dt,
				0.0, EJointMotionType::Locked, ConstraintArm0, ConstraintArm1);
		}
	}
}

void FPBDJointCachedSolver::InitLockedPositionConstraintSimd(
	const FPBDJointSettings& JointSettings,
	const FReal Dt,
	const TVec3<EJointMotionType>& LinearMotion)
{
	FVec3 ConstraintArm0 = ConnectorXs[1] - CurrentPs[0];
	const FVec3 ConstraintArm1 = ConnectorXs[1] - CurrentPs[1];

	const FVec3 DX = ConnectorXs[1] - ConnectorXs[0];
	const FMatrix33 R0M = ConnectorRs[0].ToMatrix();
	FVec3 CX = FVec3::ZeroVector;

	FRealSingle AddedIMass = FRealSingle(InvM(0) + InvM(1));
	FVec3f HardIM(AddedIMass);
	FVec3f LocalDeltas;

	// For a locked constraint we try to match an exact constraint, 
	// it is why we are adding back the constraint projection along each axis.
	// If the 3 axis are locked the constraintarm0 is then ConnectorXs[0] - CurrentPs[0];
	for (int32 ConstraintIndex = 0; ConstraintIndex < 3; ++ConstraintIndex)
	{
		check(LinearMotion[ConstraintIndex] == EJointMotionType::Locked);
		const FVec3 ConstraintAxis = R0M.GetAxis(ConstraintIndex);
		CX[ConstraintIndex] = FVec3::DotProduct(DX, ConstraintAxis);
		ConstraintArm0 -= ConstraintAxis * CX[ConstraintIndex];


		const FVec3 LocalAxis = (CX[ConstraintIndex] < 0.0f) ? -ConstraintAxis : ConstraintAxis;
		LocalDeltas[ConstraintIndex] = FRealSingle((CX[ConstraintIndex] < 0.0) ? -CX[ConstraintIndex] : CX[ConstraintIndex]);

		PositionConstraints.Simd.ConstraintAxis[ConstraintIndex] = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(LocalAxis[0], LocalAxis[1], LocalAxis[2], 0.0f));
	}
	PositionConstraints.Simd.ConstraintCX = MakeVectorRegisterFloat(LocalDeltas[0], LocalDeltas[1], LocalDeltas[2], 0.0f);

	PositionConstraints.Simd.ConstraintArms[0] = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(ConstraintArm0[0], ConstraintArm0[1], ConstraintArm0[2], 0.0f));
	PositionConstraints.Simd.ConstraintArms[1] = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(ConstraintArm1[0], ConstraintArm1[1], ConstraintArm1[2], 0.0f));

	for (int32 ConstraintIndex = 0; ConstraintIndex < 3; ++ConstraintIndex)
	{
		const VectorRegister4Float AngularAxis0 = VectorCross(PositionConstraints.Simd.ConstraintArms[0], PositionConstraints.Simd.ConstraintAxis[ConstraintIndex]);
		const VectorRegister4Float IA0 = Private::VectorMatrixMultiply(AngularAxis0, InvI(0));

		const VectorRegister4Float AngularAxis1 = VectorCross(PositionConstraints.Simd.ConstraintArms[1], PositionConstraints.Simd.ConstraintAxis[ConstraintIndex]);
		const VectorRegister4Float IA1 = Private::VectorMatrixMultiply(AngularAxis1, InvI(1));

		const FRealSingle II0 = VectorDot3Scalar(AngularAxis0, IA0);
		const FRealSingle II1 = VectorDot3Scalar(AngularAxis1, IA1);
		HardIM[ConstraintIndex] += II0 + II1;
		PositionConstraints.Simd.ConstraintDRAxis[ConstraintIndex][0] = IA0;
		PositionConstraints.Simd.ConstraintDRAxis[ConstraintIndex][1] = VectorNegate(IA1);
	}
	PositionConstraints.Simd.ConstraintHardIM = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(HardIM[0], HardIM[1], HardIM[2], 0.0f));

}

void FPBDJointCachedSolver::InitSphericalPositionConstraint(
	const FPBDJointSettings& JointSettings,
	const FReal Dt)
{
	FVec3 SphereAxis0;
	FReal SphereDelta0;
	FPBDJointUtilities::GetSphericalAxisDelta(ConnectorXs[0], ConnectorXs[1], SphereAxis0, SphereDelta0);

	const FVec3 SphereAxis1 = (FMath::Abs(FMath::Abs(SphereAxis0.Dot(FVec3(1, 0, 0)) - 1.0)) > UE_SMALL_NUMBER) ? SphereAxis0.Cross(FVec3(1, 0, 0)) :
		(FMath::Abs(FMath::Abs(SphereAxis0.Dot(FVec3(0, 1, 0)) - 1.0)) > UE_SMALL_NUMBER) ? SphereAxis0.Cross(FVec3(0, 1, 0)) : SphereAxis0.Cross(FVec3(0, 0, 1));
	const FVec3 SphereAxis2 = SphereAxis0.Cross(SphereAxis1);

	// Using Connector1 for both conserves angular momentum and avoid having 
	// too much torque applied onto the COM. But can only be used for limited constraints
	const FVec3 ConstraintArm0 = ConnectorXs[1] - CurrentPs[0];
	const FVec3 ConstraintArm1 = ConnectorXs[1] - CurrentPs[1];

	SetInitConstraintVelocity(ConstraintArm0, ConstraintArm1);

	InitPositionConstraintDatas(0, SphereAxis0, SphereDelta0, JointSettings.LinearRestitution, Dt,
		JointSettings.LinearLimit, EJointMotionType::Limited, ConstraintArm0, ConstraintArm1);

	// SphereAxis0 being the direction axis, the geometric error for the 2 other axis are 0
	// We need to add these 2 constraints for a linear solver to avoid drifting away in
	//  the other directions while solving. For a non linear solver since we are recomputing 
	// the main direction at each time we don't need that
	InitPositionConstraintDatas(1, SphereAxis1, 0.0, JointSettings.LinearRestitution, Dt,
		JointSettings.LinearLimit, EJointMotionType::Limited, ConstraintArm0, ConstraintArm1);

	InitPositionConstraintDatas(2, SphereAxis2, 0.0, JointSettings.LinearRestitution, Dt,
		JointSettings.LinearLimit, EJointMotionType::Limited, ConstraintArm0, ConstraintArm1);
}

void FPBDJointCachedSolver::InitCylindricalPositionConstraint(
	const FPBDJointSettings& JointSettings,
	const FReal Dt,
	const int32 AxisIndex)
{
	FVec3 PlaneAxis, RadialAxis0;
	FReal PlaneDelta, RadialDelta0;
	FPBDJointUtilities::GetCylindricalAxesDeltas(ConnectorRs[0], ConnectorXs[0], ConnectorXs[1],
		AxisIndex, PlaneAxis, PlaneDelta, RadialAxis0, RadialDelta0);

	const FVec3 RadialAxis1 = PlaneAxis.Cross(RadialAxis0);
	const FReal RadialDelta1 = (ConnectorXs[1] - ConnectorXs[0]).Dot(RadialAxis1);

	const FVec3 ConstraintArm0 = ConnectorXs[1] - CurrentPs[0];
	const FVec3 ConstraintArm1 = ConnectorXs[1] - CurrentPs[1];

	SetInitConstraintVelocity(ConstraintArm0, ConstraintArm1);

	InitPositionConstraintDatas((AxisIndex + 1) % 3, RadialAxis0, RadialDelta0, JointSettings.LinearRestitution, Dt,
		JointSettings.LinearLimit, EJointMotionType::Limited, ConstraintArm0, ConstraintArm1);

	InitPositionConstraintDatas((AxisIndex + 2) % 3, RadialAxis1, RadialDelta1, JointSettings.LinearRestitution, Dt,
		JointSettings.LinearLimit, EJointMotionType::Limited, ConstraintArm0, ConstraintArm1);
}

void FPBDJointCachedSolver::InitPlanarPositionConstraint(
	const FPBDJointSettings& JointSettings,
	const FReal Dt,
	const int32 AxisIndex)
{
	FVec3 PlaneAxis;
	FReal PlaneDelta;
	FPBDJointUtilities::GetPlanarAxisDelta(ConnectorRs[0], ConnectorXs[0], ConnectorXs[1], AxisIndex, PlaneAxis, PlaneDelta);

	const FVec3 ConstraintArm0 = ConnectorXs[1] - CurrentPs[0];
	const FVec3 ConstraintArm1 = ConnectorXs[1] - CurrentPs[1];

	SetInitConstraintVelocity(ConstraintArm0, ConstraintArm1);

	InitPositionConstraintDatas(AxisIndex, PlaneAxis, PlaneDelta, JointSettings.LinearRestitution, Dt,
		JointSettings.LinearLimit, EJointMotionType::Limited, ConstraintArm0, ConstraintArm1);
}

/** APPLY POSITION CONSTRAINT *****************************************************************************************/

void FPBDJointCachedSolver::ApplyPositionConstraints(
	const FReal Dt)
{
	if (PositionConstraints.bUseSimd)
	{
		ApplyPositionConstraintsSimd(Dt);
	}
	else
	{
		for (int32 ConstraintIndex = 0; ConstraintIndex < 3; ++ConstraintIndex)
		{
			if (PositionConstraints.GetValidDatas(ConstraintIndex))
			{
				ApplyAxisPositionConstraint(ConstraintIndex, Dt);
			}
		}
	}
}

void FPBDJointCachedSolver::SolvePositionConstraintDelta(
	const int32 ConstraintIndex,
	const FReal DeltaLambda,
	const FAxisConstraintDatas& ConstraintDatas)
{
	const FVec3 DX = ConstraintDatas.Data.ConstraintAxis[ConstraintIndex] * DeltaLambda;

	if (Body(0).IsDynamic())
	{
		const FVec3 DP0 = InvM(0) * DX;
		const FVec3 DR0 = ConstraintDatas.Data.ConstraintDRAxis[ConstraintIndex][0] * DeltaLambda;
		ApplyPositionDelta(0, DP0);
		ApplyRotationDelta(0, DR0);
	}
	if (Body(1).IsDynamic())
	{
		const FVec3 DP1 = -InvM(1) * DX;
		const FVec3 DR1 = ConstraintDatas.Data.ConstraintDRAxis[ConstraintIndex][1] * DeltaLambda;
		ApplyPositionDelta(1, DP1);
		ApplyRotationDelta(1, DR1);
	}

	++NumActiveConstraints;
}

void FPBDJointCachedSolver::SolvePositionConstraintHard(
	const int32 ConstraintIndex,
	const FReal DeltaConstraint)
{
	const FReal DeltaLambda = SolverStiffness * PositionConstraints.Data.ConstraintHardStiffness[ConstraintIndex] * DeltaConstraint /
		PositionConstraints.Data.ConstraintHardIM[ConstraintIndex];

	PositionConstraints.Data.ConstraintLambda[ConstraintIndex] += DeltaLambda;
	SolvePositionConstraintDelta(ConstraintIndex, DeltaLambda, PositionConstraints);
}

void FPBDJointCachedSolver::SolvePositionConstraintSoft(
	const int32 ConstraintIndex,
	const FReal DeltaConstraint,
	const FReal Dt,
	const FReal TargetVel)
{
	check(PositionConstraints.bUseSimd == false);
	FReal VelDt = 0;
	if (PositionConstraints.Data.ConstraintSoftDamping[ConstraintIndex] > UE_KINDA_SMALL_NUMBER)
	{
		const FVec3 V0Dt = FVec3::CalculateVelocity(InitConnectorXs[0], ConnectorXs[0] + Body(0).DP() + FVec3::CrossProduct(Body(0).DQ(), PositionConstraints.Data.ConstraintArms[ConstraintIndex][0]), 1.0f);
		const FVec3 V1Dt = FVec3::CalculateVelocity(InitConnectorXs[1], ConnectorXs[1] + Body(1).DP() + FVec3::CrossProduct(Body(1).DQ(), PositionConstraints.Data.ConstraintArms[ConstraintIndex][1]), 1.0f);
		VelDt = TargetVel * Dt + FVec3::DotProduct(V0Dt - V1Dt, PositionConstraints.Data.ConstraintAxis[ConstraintIndex]);
	}

	const FReal DeltaLambda = SolverStiffness * (PositionConstraints.Data.ConstraintSoftStiffness[ConstraintIndex] * DeltaConstraint - PositionConstraints.Data.ConstraintSoftDamping[ConstraintIndex] * VelDt - PositionConstraints.Data.ConstraintLambda[ConstraintIndex]) /
		PositionConstraints.Data.ConstraintSoftIM[ConstraintIndex];
	PositionConstraints.Data.ConstraintLambda[ConstraintIndex] += DeltaLambda;

	SolvePositionConstraintDelta(ConstraintIndex, DeltaLambda, PositionConstraints);
}

void FPBDJointCachedSolver::ApplyAxisPositionConstraint(
	const int32 ConstraintIndex, const FReal Dt)
{

	check(PositionConstraints.bUseSimd == false);
	const FVec3 CX = Body(1).DP() - Body(0).DP() +
		FVec3::CrossProduct(Body(1).DQ(), PositionConstraints.Data.ConstraintArms[ConstraintIndex][1]) -
		FVec3::CrossProduct(Body(0).DQ(), PositionConstraints.Data.ConstraintArms[ConstraintIndex][0]);

	FReal DeltaPosition = PositionConstraints.Data.ConstraintCX[ConstraintIndex] + FVec3::DotProduct(CX, PositionConstraints.Data.ConstraintAxis[ConstraintIndex]);

	bool NeedsSolve = false;
	if (PositionConstraints.GetLimitsCheck(ConstraintIndex))
	{
		if (DeltaPosition > PositionConstraints.Data.ConstraintLimits[ConstraintIndex])
		{
			DeltaPosition -= PositionConstraints.Data.ConstraintLimits[ConstraintIndex];
			NeedsSolve = true;
		}
		else if (DeltaPosition < -PositionConstraints.Data.ConstraintLimits[ConstraintIndex])
		{
			DeltaPosition += PositionConstraints.Data.ConstraintLimits[ConstraintIndex];
			NeedsSolve = true;
		}
	}
	if (!PositionConstraints.GetLimitsCheck(ConstraintIndex) || (PositionConstraints.GetLimitsCheck(ConstraintIndex) && NeedsSolve && FMath::Abs(DeltaPosition) > PositionTolerance))
	{
		if ((PositionConstraints.GetMotionType(ConstraintIndex) == EJointMotionType::Limited) && PositionConstraints.GetSoftLimit(ConstraintIndex))
		{
			SolvePositionConstraintSoft(ConstraintIndex, DeltaPosition, Dt, 0.0f);
		}
		else if (PositionConstraints.GetMotionType(ConstraintIndex) != EJointMotionType::Free)
		{
			SolvePositionConstraintHard(ConstraintIndex, DeltaPosition);
		}
	}
}

void FPBDJointCachedSolver::ApplyPositionConstraintsSimd(
	const FReal Dt)
{
	const VectorRegister4Float Body0DP = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(Body(0).DP()[0], Body(0).DP()[1], Body(0).DP()[2], 0.0f));
	const VectorRegister4Float Body1DP = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(Body(1).DP()[0], Body(1).DP()[1], Body(1).DP()[2], 0.0f));

	const VectorRegister4Float Body0DQ = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(Body(0).DQ()[0], Body(0).DQ()[1], Body(0).DQ()[2], 0.0f));
	const VectorRegister4Float Body1DQ = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(Body(1).DQ()[0], Body(1).DQ()[1], Body(1).DQ()[2], 0.0f));

	const VectorRegister4Float DPDiff = VectorSubtract(Body1DP, Body0DP);
	const VectorRegister4Float Cross1 = VectorCross(Body1DQ, PositionConstraints.Simd.ConstraintArms[1]);
	const VectorRegister4Float Cross0 = VectorCross(Body0DQ, PositionConstraints.Simd.ConstraintArms[0]);
	const VectorRegister4Float CrossDiff = VectorSubtract(Cross1, Cross0);
	const VectorRegister4Float CX = VectorAdd(DPDiff, CrossDiff);

	VectorRegister4Float DeltaPositions[3];
	for (int32 CIndex = 0; CIndex < 3; ++CIndex)
	{
		DeltaPositions[CIndex] = Private::VectorDot3FastX(CX, PositionConstraints.Simd.ConstraintAxis[CIndex]);
	}
	VectorRegister4Float DeltaPosition = VectorUnpackLo(DeltaPositions[0], DeltaPositions[1]);
	DeltaPosition = VectorMoveLh(DeltaPosition, DeltaPositions[2]);

	DeltaPosition = VectorAdd(DeltaPosition, PositionConstraints.Simd.ConstraintCX);

	const FRealSingle SolverStiffnessf = FRealSingle(SolverStiffness);
	VectorRegister4Float Stiffness = VectorLoadFloat1(&SolverStiffnessf);

	Stiffness = VectorMultiply(Stiffness, PositionConstraints.Simd.ConstraintHardStiffness);
	DeltaPosition = VectorMultiply(Stiffness, DeltaPosition);
	const VectorRegister4Float DeltaLambda = VectorDivide(DeltaPosition, PositionConstraints.Simd.ConstraintHardIM);
	PositionConstraints.Simd.ConstraintLambda = VectorAdd(PositionConstraints.Simd.ConstraintLambda, DeltaLambda);

	VectorRegister4Float DLambda[3];
	DLambda[0] = VectorReplicate(DeltaLambda, 0);
	DLambda[1] = VectorReplicate(DeltaLambda, 1);
	DLambda[2] = VectorReplicate(DeltaLambda, 2);

	VectorRegister4Float DX[3];
	for (int32 CIndex = 0; CIndex < 3; CIndex++)
	{
		DX[CIndex] = VectorMultiply(PositionConstraints.Simd.ConstraintAxis[CIndex], DLambda[CIndex]);
	}

	if (Body(0).IsDynamic())
	{
		const FRealSingle Inv0f = FRealSingle(InvM(0));
		const VectorRegister4Float Inv0 = VectorLoadFloat1(&Inv0f);

		VectorRegister4Float DP0 = VectorZero();
		VectorRegister4Float DR0 = VectorZero();
		for (int32 CIndex = 0; CIndex < 3; CIndex++)
		{
			DP0 = VectorMultiplyAdd(Inv0, DX[CIndex], DP0);
			DR0 = VectorMultiplyAdd(PositionConstraints.Simd.ConstraintDRAxis[CIndex][0], DLambda[CIndex], DR0);
		}
		FVec3f DP0f;
		VectorStoreFloat3(DP0, &DP0f[0]);
		ApplyPositionDelta(0, DP0f);
		FVec3f DR0f;
		VectorStoreFloat3(DR0, &DR0f[0]);
		ApplyRotationDelta(0, DR0f);
	}
	if (Body(1).IsDynamic())
	{
		const FRealSingle Inv1f = FRealSingle(InvM(1));
		const VectorRegister4Float Inv1 = VectorLoadFloat1(&Inv1f);

		VectorRegister4Float DP1 = VectorZero();
		VectorRegister4Float DR1 = VectorZero();
		for (int32 CIndex = 0; CIndex < 3; CIndex++)
		{
			DP1 = VectorSubtract(DP1, VectorMultiply(Inv1, DX[CIndex]));
			DR1 = VectorMultiplyAdd(PositionConstraints.Simd.ConstraintDRAxis[CIndex][1], DLambda[CIndex], DR1);
		}
		FVec3f DP1f;
		VectorStoreFloat3(DP1, &DP1f[0]);
		ApplyPositionDelta(1, DP1f);
		FVec3f DR1f;
		VectorStoreFloat3(DR1, &DR1f[0]);
		ApplyRotationDelta(1, DR1f);
	}
	NumActiveConstraints += 3;
}

/** APPLY LINEAR VELOCITY *********************************************************************************************/

void FPBDJointCachedSolver::ApplyLinearVelocityConstraints()
{
	if (PositionConstraints.bUseSimd)
	{
		ApplyVelocityConstraintSimd();
	}
	else
	{
		for (int32 ConstraintIndex = 0; ConstraintIndex < 3; ++ConstraintIndex)
		{
			if (PositionConstraints.GetValidDatas(ConstraintIndex) && !PositionConstraints.GetSoftLimit(ConstraintIndex))
			{
				ApplyAxisVelocityConstraint(ConstraintIndex);
			}
		}
	}
}

void FPBDJointCachedSolver::SolveLinearVelocityConstraint(
	const int32 ConstraintIndex,
	const FReal TargetVel)
{
	check(PositionConstraints.bUseSimd == false);
	const FVec3 CV0 = V(0) + FVec3::CrossProduct(W(0), PositionConstraints.Data.ConstraintArms[ConstraintIndex][0]);
	const FVec3 CV1 = V(1) + FVec3::CrossProduct(W(1), PositionConstraints.Data.ConstraintArms[ConstraintIndex][1]);
	const FVec3 CV = CV1 - CV0;

	const FReal DeltaLambda = SolverStiffness * PositionConstraints.Data.ConstraintHardStiffness[ConstraintIndex] *
		(FVec3::DotProduct(CV, PositionConstraints.Data.ConstraintAxis[ConstraintIndex]) - TargetVel) / PositionConstraints.Data.ConstraintHardIM[ConstraintIndex];

	// @todo(chaos): We should be adding to the net positional impulse here
	//PositionConstraints.Data.ConstraintLambda[ConstraintIndex] += DeltaLambda * Dt;

	const FVec3 MDV = DeltaLambda * PositionConstraints.Data.ConstraintAxis[ConstraintIndex];

	if (Body(0).IsDynamic())
	{
		const FVec3 DV0 = InvM(0) * MDV;
		const FVec3 DW0 = PositionConstraints.Data.ConstraintDRAxis[ConstraintIndex][0] * DeltaLambda;

		Body(0).ApplyVelocityDelta(DV0, DW0);
	}
	if (Body(1).IsDynamic())
	{
		const FVec3 DV1 = -InvM(1) * MDV;
		const FVec3 DW1 = PositionConstraints.Data.ConstraintDRAxis[ConstraintIndex][1] * DeltaLambda;

		Body(1).ApplyVelocityDelta(DV1, DW1);
	}
}

void FPBDJointCachedSolver::ApplyAxisVelocityConstraint(
	const int32 ConstraintIndex)
{
	check(PositionConstraints.bUseSimd == false);
	// Apply restitution for limited joints when we have exceeded the limits
	// We also drive the velocity to zero for locked constraints (ignoring restitution)
	if (FMath::Abs(PositionConstraints.Data.ConstraintLambda[ConstraintIndex]) > UE_SMALL_NUMBER)
	{
		FReal TargetVel = 0.0f;
		const FReal Restitution = PositionConstraints.ConstraintRestitution[ConstraintIndex];
		const bool bIsLimited = (PositionConstraints.GetMotionType(ConstraintIndex) == EJointMotionType::Limited);
		if (bIsLimited && (Restitution != 0.0f))
		{
			const FReal InitVel = InitConstraintAxisLinearVelocities[ConstraintIndex];
			const FReal Threshold = Chaos_Joint_LinearVelocityThresholdToApplyRestitution;
			TargetVel = (InitVel > Threshold) ? -Restitution * InitVel : 0.0f;
		}
		SolveLinearVelocityConstraint(ConstraintIndex, TargetVel);
	}
}


void FPBDJointCachedSolver::ApplyVelocityConstraintSimd()
{
	const VectorRegister4Float IsGTEps = VectorCompareGT(VectorAbs(PositionConstraints.Simd.ConstraintLambda), GlobalVectorConstants::SmallNumber);

	if (VectorMaskBits(IsGTEps))
	{
		const FVec3& V0d = V(0);
		const VectorRegister4Float V0 = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(V0d[0], V0d[1], V0d[2], 0.0f));
		const FVec3& V1d = V(1);
		const VectorRegister4Float V1 = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(V1d[0], V1d[1], V1d[2], 0.0f));
		const FVec3& W0d = W(0);
		const VectorRegister4Float W0 = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(W0d[0], W0d[1], W0d[2], 0.0f));
		const FVec3& W1d = W(1);
		const VectorRegister4Float W1 = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(W1d[0], W1d[1], W1d[2], 0.0f));

		const FRealSingle SolverStiffnessf = FRealSingle(SolverStiffness);
		VectorRegister4Float Stiffness = VectorLoadFloat1(&SolverStiffnessf);
		Stiffness = VectorMultiply(Stiffness, PositionConstraints.Simd.ConstraintHardStiffness);

		const VectorRegister4Float CV0 = VectorAdd(V0, VectorCross(W0, PositionConstraints.Simd.ConstraintArms[0]));
		const VectorRegister4Float CV1 = VectorAdd(V1, VectorCross(W1, PositionConstraints.Simd.ConstraintArms[1]));
		const VectorRegister4Float CV = VectorSubtract(CV1, CV0);

		VectorRegister4Float ProjVs[3];
		for (int32 CIndex = 0; CIndex < 3; CIndex++)
		{
			ProjVs[CIndex] = Private::VectorDot3FastX(CV, PositionConstraints.Simd.ConstraintAxis[CIndex]);
		}
		VectorRegister4Float ProjV = VectorUnpackLo(ProjVs[0], ProjVs[1]);
		ProjV = VectorMoveLh(ProjV, ProjVs[2]);

		const VectorRegister4Float DeltaLambda = VectorDivide(VectorMultiply(Stiffness, ProjV), PositionConstraints.Simd.ConstraintHardIM);

		VectorRegister4Float DeltaLambdas[3];
		DeltaLambdas[0] = VectorReplicate(DeltaLambda, 0);
		DeltaLambdas[1] = VectorReplicate(DeltaLambda, 1);
		DeltaLambdas[2] = VectorReplicate(DeltaLambda, 2);

		if (Body(0).IsDynamic())
		{
			const FRealSingle Inv0f = FRealSingle(InvM(0));
			const VectorRegister4Float InvM0 = VectorLoadFloat1(&Inv0f);
			VectorRegister4Float DV0 = VectorZeroFloat();
			VectorRegister4Float DW0 = VectorZeroFloat();
			for (int32 CIndex = 0; CIndex < 3; CIndex++)
			{
				DV0 = VectorMultiplyAdd(InvM0, VectorMultiply(DeltaLambdas[CIndex], PositionConstraints.Simd.ConstraintAxis[CIndex]), DV0);
				DW0 = VectorMultiplyAdd(PositionConstraints.Simd.ConstraintDRAxis[CIndex][0], DeltaLambdas[CIndex], DW0);
			}
			FVec3f DV0f;
			VectorStoreFloat3(DV0, &DV0f[0]);
			FVec3f DW0f;
			VectorStoreFloat3(DW0, &DW0f[0]);
			Body(0).ApplyVelocityDelta(FVec3(DV0f), FVec3(DW0f));
		}
		if (Body(1).IsDynamic())
		{
			const FRealSingle OppInv1f = -FRealSingle(InvM(1));
			const VectorRegister4Float OppInvM1 = VectorLoadFloat1(&OppInv1f);
			VectorRegister4Float DV1 = VectorZeroFloat();
			VectorRegister4Float DW1 = VectorZeroFloat();
			for (int32 CIndex = 0; CIndex < 3; CIndex++)
			{
				DV1 = VectorMultiplyAdd(OppInvM1, VectorMultiply(DeltaLambdas[CIndex], PositionConstraints.Simd.ConstraintAxis[CIndex]), DV1);
				DW1 = VectorMultiplyAdd(PositionConstraints.Simd.ConstraintDRAxis[CIndex][1], DeltaLambdas[CIndex], DW1);
			}
			FVec3f DV1f;
			VectorStoreFloat3(DV1, &DV1f[0]);
			FVec3f DW1f;
			VectorStoreFloat3(DW1, &DW1f[0]);
			Body(1).ApplyVelocityDelta(FVec3(DV1f), FVec3(DW1f));
		}
	}
}

/** UTILS FOR ROTATION CONSTRAINT **************************************************************************************/

bool ExtractAngularMotion(const FPBDJointSettings& JointSettings,
	TVec3<bool>& bAngularLocked, TVec3<bool>& bAngularLimited, TVec3<bool>& bAngularFree)
{
	bool bHasRotationConstraints =
		(JointSettings.AngularMotionTypes[0] != EJointMotionType::Free)
		|| (JointSettings.AngularMotionTypes[1] != EJointMotionType::Free)
		|| (JointSettings.AngularMotionTypes[2] != EJointMotionType::Free);
	if (!bHasRotationConstraints)
	{
		return false;
	}

	const TVec3<EJointMotionType>& AngularMotion = JointSettings.AngularMotionTypes;
	bAngularLocked =
	{
		(AngularMotion[0] == EJointMotionType::Locked),
		(AngularMotion[1] == EJointMotionType::Locked),
		(AngularMotion[2] == EJointMotionType::Locked),
	};
	bAngularLimited =
	{
		(AngularMotion[0] == EJointMotionType::Limited),
		(AngularMotion[1] == EJointMotionType::Limited),
		(AngularMotion[2] == EJointMotionType::Limited),
	};
	bAngularFree =
	{
		(AngularMotion[0] == EJointMotionType::Free),
		(AngularMotion[1] == EJointMotionType::Free),
		(AngularMotion[2] == EJointMotionType::Free),
	};
	return true;
}

/** INIT ROTATION CONSTRAINT ******************************************************************************************/

void FPBDJointCachedSolver::InitRotationConstraints(
	const FReal Dt,
	const FPBDJointSolverSettings& SolverSettings,
	const FPBDJointSettings& JointSettings,
	const bool bResetLambdas)
{
	RotationConstraints.SetValidDatas(0, false);
	RotationConstraints.SetValidDatas(1, false);
	RotationConstraints.SetValidDatas(2, false);
	RotationConstraints.bUseSimd = false;


	TVec3<bool> bAngularLocked, bAngularLimited, bAngularFree;
	if (!ExtractAngularMotion(JointSettings, bAngularLocked, bAngularLimited, bAngularFree))
		return;

	RotationConstraints.bAccelerationMode = FPBDJointUtilities::GetAngularSoftAccelerationMode(SolverSettings, JointSettings);

	RotationConstraints.bUseSimd = bUseSimd && bAngularLimited[0] && bAngularLimited[1] && bAngularLimited[2] &&
		FPBDJointUtilities::GetSoftTwistLimitEnabled(SolverSettings, JointSettings) && FPBDJointUtilities::GetSoftSwingLimitEnabled(SolverSettings, JointSettings);

	if (RotationConstraints.bUseSimd)
	{
		RotationConstraints.SettingsSoftStiffness = FVec3(FPBDJointUtilities::GetSoftTwistStiffness(SolverSettings, JointSettings), FPBDJointUtilities::GetSoftSwingStiffness(SolverSettings, JointSettings), FPBDJointUtilities::GetSoftSwingStiffness(SolverSettings, JointSettings));
		RotationConstraints.SettingsSoftDamping = FVec3(FPBDJointUtilities::GetSoftTwistDamping(SolverSettings, JointSettings), FPBDJointUtilities::GetSoftSwingDamping(SolverSettings, JointSettings), FPBDJointUtilities::GetSoftSwingDamping(SolverSettings, JointSettings));

		if (bResetLambdas)
		{
			RotationConstraints.Simd.ConstraintLambda = VectorZeroFloat();
		}

		RotationConstraints.Simd.ConstraintHardStiffness = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(FPBDJointUtilities::GetTwistStiffness(SolverSettings, JointSettings),
			FPBDJointUtilities::GetSwingStiffness(SolverSettings, JointSettings), FPBDJointUtilities::GetSwingStiffness(SolverSettings, JointSettings), 0.0f));
		InitRotationConstraintsSimd(JointSettings, FRealSingle(Dt));
	}
	else
	{
		const int32 TW = (int32)EJointAngularConstraintIndex::Twist;
		const int32 S1 = (int32)EJointAngularConstraintIndex::Swing1;
		const int32 S2 = (int32)EJointAngularConstraintIndex::Swing2;

		RotationConstraints.InitDatas(TW, FPBDJointUtilities::GetSoftTwistLimitEnabled(SolverSettings, JointSettings) && !bAngularLocked[TW],
			FPBDJointUtilities::GetSoftTwistStiffness(SolverSettings, JointSettings),
			FPBDJointUtilities::GetSoftTwistDamping(SolverSettings, JointSettings),
			FPBDJointUtilities::GetTwistStiffness(SolverSettings, JointSettings),
			bResetLambdas);

		RotationConstraints.InitDatas(S1, FPBDJointUtilities::GetSoftSwingLimitEnabled(SolverSettings, JointSettings) && !bAngularLocked[S1],
			FPBDJointUtilities::GetSoftSwingStiffness(SolverSettings, JointSettings),
			FPBDJointUtilities::GetSoftSwingDamping(SolverSettings, JointSettings),
			FPBDJointUtilities::GetSwingStiffness(SolverSettings, JointSettings),
			bResetLambdas);

		RotationConstraints.InitDatas(S2, FPBDJointUtilities::GetSoftSwingLimitEnabled(SolverSettings, JointSettings) && !bAngularLocked[S2],
			FPBDJointUtilities::GetSoftSwingStiffness(SolverSettings, JointSettings),
			FPBDJointUtilities::GetSoftSwingDamping(SolverSettings, JointSettings),
			FPBDJointUtilities::GetSwingStiffness(SolverSettings, JointSettings),
			bResetLambdas);



		const FVec3 Twist0 = ConnectorRs[0] * FJointConstants::TwistAxis();
		const FVec3 Twist1 = ConnectorRs[1] * FJointConstants::TwistAxis();
		const bool bDegenerate = (FVec3::DotProduct(Twist0, Twist1) < Chaos_Joint_DegenerateRotationLimit);

		// Apply twist constraint
		// NOTE: Cannot calculate twist angle at 180degree swing
		if (SolverSettings.bEnableTwistLimits)
		{
			if (bAngularLimited[TW] && !bDegenerate)
			{
				InitTwistConstraint(JointSettings, Dt);
			}
		}

		// Apply swing constraints
		// NOTE: Cannot separate swing angles at 180degree swing (but we can still apply locks)
		if (SolverSettings.bEnableSwingLimits)
		{
			if (bAngularLimited[S1] && bAngularLimited[S2])
			{
				// When using non linear solver, the cone swing direction could change at each iteration
				// stabilizing the solver. In the linear case we need to constraint along the 2 directions
				// for better stability
				InitPyramidSwingConstraint(JointSettings, Dt, true, true);
			}
			else if (bAngularLimited[S1] && bAngularLocked[S2])
			{
				if (!bDegenerate)
				{
					InitPyramidSwingConstraint(JointSettings, Dt, true, false);
				}
			}
			else if (bAngularLimited[S1] && bAngularFree[S2])
			{
				if (!bDegenerate)
				{
					InitDualConeSwingConstraint(JointSettings, Dt, EJointAngularConstraintIndex::Swing1);
				}
			}
			else if (bAngularLocked[S1] && bAngularLimited[S2])
			{
				if (!bDegenerate)
				{
					InitPyramidSwingConstraint(JointSettings, Dt, false, true);
				}
			}
			else if (bAngularFree[S1] && bAngularLimited[S2])
			{
				if (!bDegenerate)
				{
					InitDualConeSwingConstraint(JointSettings, Dt, EJointAngularConstraintIndex::Swing2);
				}
			}
		}


		// Note: single-swing locks are already handled above so we only need to do something here if both are locked
		const bool bLockedTwist = SolverSettings.bEnableTwistLimits && bAngularLocked[TW];
		const bool bLockedSwing1 = SolverSettings.bEnableSwingLimits && bAngularLocked[S1];
		const bool bLockedSwing2 = SolverSettings.bEnableSwingLimits && bAngularLocked[S2];
		if (bLockedTwist || bLockedSwing1 || bLockedSwing2)
		{
			InitLockedRotationConstraints(JointSettings, Dt, bLockedTwist, bLockedSwing1, bLockedSwing2);
		}
	}
	// Todo at this point Motion Type has never been initialized so it will be always Free here. 
	// This cause restitution to be always disabled and maybe more... 
}

void FPBDJointCachedSolver::InitRotationConstraintsSimd(
	const FPBDJointSettings& JointSettings,
	const FRealSingle Dtf)
{
	const FVec3 Twist0 = ConnectorRs[0] * FJointConstants::TwistAxis();
	const FVec3 Twist1 = ConnectorRs[1] * FJointConstants::TwistAxis();
	const bool bDegenerate = (FVec3::DotProduct(Twist0, Twist1) < Chaos_Joint_DegenerateRotationLimit);

	FVec3 Axes[3];
	FReal Angles[3];

	FPBDJointUtilities::GetTwistAxisAngle(ConnectorRs[0], ConnectorRs[1], Axes[0], Angles[0]);

	// Project the angle directly to avoid checking the limits during the solve.

	// Decompose rotation of body 1 relative to body 0 into swing and twist rotations, assuming twist is X axis
	FRotation3 R01Twist, R01Swing;
	FPBDJointUtilities::DecomposeSwingTwistLocal(ConnectorRs[0], ConnectorRs[1], R01Swing, R01Twist);
	const FRotation3 R0Swing = ConnectorRs[0] * R01Swing;
	Axes[2] = R0Swing * FJointConstants::Swing1Axis();
	Angles[2] = 4.0 * FMath::Atan2(R01Swing.Z, (FReal)(1. + R01Swing.W));
	Axes[1] = R0Swing * FJointConstants::Swing2Axis();
	Angles[1] = 4.0 * FMath::Atan2(R01Swing.Y, (FReal)(1. + R01Swing.W));

	RotationConstraints.ConstraintRestitution = FVec3(JointSettings.TwistRestitution, JointSettings.SwingRestitution, JointSettings.SwingRestitution);

	FVec3f ConstraintHardIM;
	FVec3f LocalAngles;

	for (int32 ConstraintIndex = 0; ConstraintIndex < 3; ConstraintIndex++)
	{
		const FVec3 LocalAxis = (Angles[ConstraintIndex] < 0.0f) ? -Axes[ConstraintIndex] : Axes[ConstraintIndex];
		LocalAngles[ConstraintIndex] = FRealSingle((Angles[ConstraintIndex] < 0.0f) ? -Angles[ConstraintIndex] : Angles[ConstraintIndex]);
		InitConstraintAxisAngularVelocities[ConstraintIndex] = FVec3::DotProduct(W(1) - W(0), LocalAxis);

		RotationConstraints.Simd.ConstraintAxis[ConstraintIndex] = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(LocalAxis[0], LocalAxis[1], LocalAxis[2], 0.0f));
		const VectorRegister4Float& Axis = RotationConstraints.Simd.ConstraintAxis[ConstraintIndex];

		const VectorRegister4Float AxisX = VectorReplicate(Axis, 0);
		const VectorRegister4Float AxisY = VectorReplicate(Axis, 1);
		const VectorRegister4Float AxisZ = VectorReplicate(Axis, 2);
		const FMatrix33& InvI0 = InvI(0);
		const VectorRegister4Float InvI00 = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(InvI0.M[0][0], InvI0.M[0][1], InvI0.M[0][2], 0.0f));
		const VectorRegister4Float InvI01 = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(InvI0.M[1][0], InvI0.M[1][1], InvI0.M[1][2], 0.0f));
		const VectorRegister4Float InvI02 = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(InvI0.M[2][0], InvI0.M[2][1], InvI0.M[2][2], 0.0f));
		const VectorRegister4Float IA0 = VectorMultiplyAdd(InvI00, AxisX, VectorMultiplyAdd(InvI01, AxisY, VectorMultiply(InvI02, AxisZ)));

		const FMatrix33& InvI1 = InvI(1);
		const VectorRegister4Float InvI10 = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(InvI1.M[0][0], InvI1.M[0][1], InvI1.M[0][2], 0.0f));
		const VectorRegister4Float InvI11 = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(InvI1.M[1][0], InvI1.M[1][1], InvI1.M[1][2], 0.0f));
		const VectorRegister4Float InvI12 = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(InvI1.M[2][0], InvI1.M[2][1], InvI1.M[2][2], 0.0f));
		const VectorRegister4Float IA1 = VectorMultiplyAdd(InvI10, AxisX, VectorMultiplyAdd(InvI11, AxisY, VectorMultiply(InvI12, AxisZ)));

		const FRealSingle II0 = VectorDot3Scalar(Axis, IA0);
		const FRealSingle II1 = VectorDot3Scalar(Axis, IA1);
		RotationConstraints.Simd.ConstraintDRAxis[ConstraintIndex][0] = IA0;
		RotationConstraints.Simd.ConstraintDRAxis[ConstraintIndex][1] = VectorNegate(IA1);

		ConstraintHardIM[ConstraintIndex] = II0 + II1;
	}

	RotationConstraints.Simd.ConstraintHardIM = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(ConstraintHardIM[0], ConstraintHardIM[1], ConstraintHardIM[2], 0.0f));
	const VectorRegister4Float SoftDamping = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(RotationConstraints.SettingsSoftDamping[0], RotationConstraints.SettingsSoftDamping[1], RotationConstraints.SettingsSoftDamping[2], 0.0f));
	const VectorRegister4Float SoftStiffness = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(RotationConstraints.SettingsSoftStiffness[0], RotationConstraints.SettingsSoftStiffness[1], RotationConstraints.SettingsSoftStiffness[2], 0.0f));

	const VectorRegister4Float Dt = VectorLoadFloat1(&Dtf);

	const VectorRegister4Float SpringMassScale = RotationConstraints.bAccelerationMode ? VectorDivide(GlobalVectorConstants::FloatOne, RotationConstraints.Simd.ConstraintHardIM) : GlobalVectorConstants::FloatOne;
	RotationConstraints.Simd.ConstraintSoftStiffness = VectorMultiply(VectorMultiply(SpringMassScale, SoftStiffness), VectorMultiply(Dt, Dt));
	RotationConstraints.Simd.ConstraintSoftDamping = bUsePositionBasedDrives ? VectorMultiply(SpringMassScale, VectorMultiply(SoftDamping, Dt)) : VectorZeroFloat();
	RotationConstraints.Simd.ConstraintSoftIM = VectorAdd(VectorMultiply(VectorAdd(RotationConstraints.Simd.ConstraintSoftStiffness, RotationConstraints.Simd.ConstraintSoftDamping), RotationConstraints.Simd.ConstraintHardIM), GlobalVectorConstants::FloatOne);

	RotationConstraints.Simd.ConstraintCX = MakeVectorRegisterFloat(LocalAngles[0], LocalAngles[1], LocalAngles[2], 0.0f);
	RotationConstraints.Simd.ConstraintLimits = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(JointSettings.AngularLimits[0], JointSettings.AngularLimits[1], JointSettings.AngularLimits[2], UE_BIG_NUMBER));
}

void FPBDJointCachedSolver::InitRotationDatasMass(
	FAxisConstraintDatas& RotationDatas,
	const int32 ConstraintIndex,
	const FReal Dt)
{
	const FVec3 IA0 = Utilities::Multiply(InvI(0), RotationDatas.Data.ConstraintAxis[ConstraintIndex]);
	const FVec3 IA1 = Utilities::Multiply(InvI(1), RotationDatas.Data.ConstraintAxis[ConstraintIndex]);
	const FReal II0 = FVec3::DotProduct(RotationDatas.Data.ConstraintAxis[ConstraintIndex], IA0);
	const FReal II1 = FVec3::DotProduct(RotationDatas.Data.ConstraintAxis[ConstraintIndex], IA1);

	RotationDatas.UpdateMass(ConstraintIndex, IA0, IA1, II0 + II1, Dt, bUsePositionBasedDrives);
}

void FPBDJointCachedSolver::InitRotationConstraintDatas(
	const FPBDJointSettings& JointSettings,
	const int32 ConstraintIndex,
	const FVec3& ConstraintAxis,
	const FReal ConstraintAngle,
	const FReal ConstraintRestitution,
	const FReal Dt,
	const bool bCheckLimit)
{
	const FVec3 LocalAxis = (ConstraintAngle < 0.0f) ? -ConstraintAxis : ConstraintAxis;
	const FReal LocalAngle = (ConstraintAngle < 0.0f) ? -ConstraintAngle : ConstraintAngle;

	RotationConstraints.UpdateDatas(ConstraintIndex, LocalAxis, LocalAngle, ConstraintRestitution, bCheckLimit);
	RotationConstraints.Data.ConstraintLimits[ConstraintIndex] = JointSettings.AngularLimits[ConstraintIndex];
	InitConstraintAxisAngularVelocities[ConstraintIndex] = FVec3::DotProduct(W(1) - W(0), LocalAxis);

	InitRotationDatasMass(RotationConstraints, ConstraintIndex, Dt);
}

void FPBDJointCachedSolver::CorrectAxisAngleConstraint(
	const FPBDJointSettings& JointSettings,
	const int32 ConstraintIndex,
	FVec3& ConstraintAxis,
	FReal& ConstraintAngle) const
{
	const FReal AngleMax = JointSettings.AngularLimits[ConstraintIndex];

	if (ConstraintAngle > AngleMax)
	{
		ConstraintAngle = ConstraintAngle - AngleMax;
	}
	else if (ConstraintAngle < -AngleMax)
	{
		// Keep Twist error positive
		ConstraintAngle = -ConstraintAngle - AngleMax;
		ConstraintAxis = -ConstraintAxis;
	}
	else
	{
		ConstraintAngle = 0;
	}
}

void FPBDJointCachedSolver::InitTwistConstraint(
	const FPBDJointSettings& JointSettings,
	const FReal Dt)
{
	FVec3 TwistAxis;
	FReal TwistAngle;
	FPBDJointUtilities::GetTwistAxisAngle(ConnectorRs[0], ConnectorRs[1], TwistAxis, TwistAngle);

	// Project the angle directly to avoid checking the limits during the solve.
	InitRotationConstraintDatas(JointSettings, (int32)EJointAngularConstraintIndex::Twist, TwistAxis, TwistAngle, JointSettings.TwistRestitution, Dt, true);
}

void FPBDJointCachedSolver::InitPyramidSwingConstraint(
	const FPBDJointSettings& JointSettings,
	const FReal Dt,
	const bool bApplySwing1,
	const bool bApplySwing2)
{
	// Decompose rotation of body 1 relative to body 0 into swing and twist rotations, assuming twist is X axis
	FRotation3 R01Twist, R01Swing;
	FPBDJointUtilities::DecomposeSwingTwistLocal(ConnectorRs[0], ConnectorRs[1], R01Swing, R01Twist);

	const FRotation3 R0Swing = ConnectorRs[0] * R01Swing;

	if (bApplySwing1)
	{
		const FVec3 SwingAxis = R0Swing * FJointConstants::Swing1Axis();
		const FReal SwingAngle = 4.0 * FMath::Atan2(R01Swing.Z, (FReal)(1. + R01Swing.W));
		InitRotationConstraintDatas(JointSettings, (int32)EJointAngularConstraintIndex::Swing1, SwingAxis, SwingAngle, JointSettings.SwingRestitution, Dt, true);
	}
	if (bApplySwing2)
	{
		const FVec3 SwingAxis = R0Swing * FJointConstants::Swing2Axis();
		const FReal SwingAngle = 4.0 * FMath::Atan2(R01Swing.Y, (FReal)(1. + R01Swing.W));
		InitRotationConstraintDatas(JointSettings, (int32)EJointAngularConstraintIndex::Swing2, SwingAxis, SwingAngle, JointSettings.SwingRestitution, Dt, true);
	}
}

void FPBDJointCachedSolver::InitConeConstraint(
	const FPBDJointSettings& JointSettings,
	const FReal Dt)
{
	FVec3 SwingAxisLocal;
	FReal SwingAngle = 0.0f;

	FPBDJointUtilities::GetEllipticalConeAxisErrorLocal(ConnectorRs[0], ConnectorRs[1], 0.0, 0.0, SwingAxisLocal, SwingAngle);
	SwingAxisLocal.SafeNormalize();

	const FVec3 SwingAxis = ConnectorRs[0] * SwingAxisLocal;
	InitRotationConstraintDatas(JointSettings, (int32)EJointAngularConstraintIndex::Swing2, SwingAxis, SwingAngle, JointSettings.SwingRestitution, Dt, true);
}

void FPBDJointCachedSolver::InitSingleLockedSwingConstraint(
	const FPBDJointSettings& JointSettings,
	const FReal Dt,
	const EJointAngularConstraintIndex SwingConstraintIndex)
{
	//NOTE: SwingAxis is not normalized in this mode. It has length Sin(SwingAngle).
	//Likewise, the SwingAngle is actually Sin(SwingAngle)
	// FVec3 SwingAxis;
	// FReal SwingAngle;
	// FPBDJointUtilities::GetLockedSwingAxisAngle(ConnectorRs[0], ConnectorRs[1], SwingConstraintIndex, SwingAxis, SwingAngle);
	//SwingAxis.SafeNormalize();

	// Using the locked swing axis angle results in potential axis switching since this axis is the result of OtherSwing x TwistAxis
	FVec3 SwingAxis;
	FReal SwingAngle;
	FPBDJointUtilities::GetSwingAxisAngle(ConnectorRs[0], ConnectorRs[1], 0.0, SwingConstraintIndex, SwingAxis, SwingAngle);

	InitRotationConstraintDatas(JointSettings, (int32)SwingConstraintIndex, SwingAxis, SwingAngle, 0.0, Dt, false);
}


void FPBDJointCachedSolver::InitDualConeSwingConstraint(
	const FPBDJointSettings& JointSettings,
	const FReal Dt,
	const EJointAngularConstraintIndex SwingConstraintIndex)

{
	FVec3 SwingAxis;
	FReal SwingAngle;
	FPBDJointUtilities::GetDualConeSwingAxisAngle(ConnectorRs[0], ConnectorRs[1], SwingConstraintIndex, SwingAxis, SwingAngle);

	InitRotationConstraintDatas(JointSettings, (int32)SwingConstraintIndex, SwingAxis, SwingAngle, JointSettings.SwingRestitution, Dt, true);

}

void FPBDJointCachedSolver::InitSwingConstraint(
	const FPBDJointSettings& JointSettings,
	const FPBDJointSolverSettings& SolverSettings,
	const FReal Dt,
	const EJointAngularConstraintIndex SwingConstraintIndex)
{
	FVec3 SwingAxis;
	FReal SwingAngle;
	FPBDJointUtilities::GetSwingAxisAngle(ConnectorRs[0], ConnectorRs[1], SolverSettings.SwingTwistAngleTolerance, SwingConstraintIndex, SwingAxis, SwingAngle);

	InitRotationConstraintDatas(JointSettings, (int32)SwingConstraintIndex, SwingAxis, SwingAngle, JointSettings.SwingRestitution, Dt, true);
}

void FPBDJointCachedSolver::InitLockedRotationConstraints(
	const FPBDJointSettings& JointSettings,
	const FReal Dt,
	const bool bApplyTwist,
	const bool bApplySwing1,
	const bool bApplySwing2)
{
	FVec3 Axis0, Axis1, Axis2;
	FPBDJointUtilities::GetLockedRotationAxes(ConnectorRs[0], ConnectorRs[1], Axis0, Axis1, Axis2);

	const FRotation3 R01 = ConnectorRs[0].Inverse() * ConnectorRs[1];

	if (bApplyTwist)
	{
		InitRotationConstraintDatas(JointSettings, (int32)EJointAngularConstraintIndex::Twist, Axis0, R01.X, 0.0, Dt, false);
	}

	if (bApplySwing1)
	{
		InitRotationConstraintDatas(JointSettings, (int32)EJointAngularConstraintIndex::Swing1, Axis2, R01.Z, 0.0, Dt, false);
	}

	if (bApplySwing2)
	{
		InitRotationConstraintDatas(JointSettings, (int32)EJointAngularConstraintIndex::Swing2, Axis1, R01.Y, 0.0, Dt, false);
	}
}

/** APPLY ROTATION CONSTRAINT ******************************************************************************************/

void FPBDJointCachedSolver::ApplyRotationConstraints(
	const FReal Dt)
{
	if (RotationConstraints.bUseSimd)
	{
		ApplyRotationSoftConstraintsSimd(Dt);
	}
	else
	{
		for (int32 ConstraintIndex = 0; ConstraintIndex < 3; ++ConstraintIndex)
		{
			if (RotationConstraints.GetValidDatas(ConstraintIndex))
			{
				ApplyRotationConstraint(ConstraintIndex, Dt);
			}
		}
	}
}

void FPBDJointCachedSolver::SolveRotationConstraintDelta(
	const int32 ConstraintIndex,
	const FReal DeltaLambda,
	const bool bIsSoftConstraint,
	const FAxisConstraintDatas& ConstraintDatas)
{
	const FVec3 DeltaImpulse = ConstraintDatas.Data.ConstraintAxis[ConstraintIndex] * DeltaLambda;
	if (Body(0).IsDynamic())
	{
		const FVec3 DR0 = !bIsSoftConstraint ? ConstraintDatas.Data.ConstraintDRAxis[ConstraintIndex][0] * DeltaLambda :
			DeltaImpulse * (FVec3::DotProduct(ConstraintDatas.Data.ConstraintAxis[ConstraintIndex], ConstraintDatas.Data.ConstraintDRAxis[ConstraintIndex][0]));
		ApplyRotationDelta(0, DR0);
	}
	if (Body(1).IsDynamic())
	{
		const FVec3 DR1 = !bIsSoftConstraint ? ConstraintDatas.Data.ConstraintDRAxis[ConstraintIndex][1] * DeltaLambda :
			DeltaImpulse * (FVec3::DotProduct(ConstraintDatas.Data.ConstraintAxis[ConstraintIndex], ConstraintDatas.Data.ConstraintDRAxis[ConstraintIndex][1]));
		ApplyRotationDelta(1, DR1);
	}
	++NumActiveConstraints;
}

void FPBDJointCachedSolver::SolveRotationConstraintHard(
	const int32 ConstraintIndex,
	const FReal DeltaConstraint)
{
	const FReal DeltaLambda = SolverStiffness * RotationConstraints.Data.ConstraintHardStiffness[ConstraintIndex] * DeltaConstraint /
		RotationConstraints.Data.ConstraintHardIM[ConstraintIndex];

	RotationConstraints.Data.ConstraintLambda[ConstraintIndex] += DeltaLambda;
	SolveRotationConstraintDelta(ConstraintIndex, DeltaLambda, false, RotationConstraints);
}

void FPBDJointCachedSolver::SolveRotationConstraintSoft(
	const int32 ConstraintIndex,
	const FReal DeltaConstraint,
	const FReal Dt,
	const FReal TargetVel)
{
	// Damping angular velocity
	FReal AngVelDt = 0;
	if (RotationConstraints.Data.ConstraintSoftDamping[ConstraintIndex] > UE_KINDA_SMALL_NUMBER)
	{
		const FVec3 W0Dt = FVec3(Body(0).DQ()) + ConnectorWDts[0];
		const FVec3 W1Dt = FVec3(Body(1).DQ()) + ConnectorWDts[1];
		AngVelDt = TargetVel * Dt + FVec3::DotProduct(RotationConstraints.Data.ConstraintAxis[ConstraintIndex], W0Dt - W1Dt);
	}

	const FReal DeltaLambda = SolverStiffness * (RotationConstraints.Data.ConstraintSoftStiffness[ConstraintIndex] * DeltaConstraint -
		RotationConstraints.Data.ConstraintSoftDamping[ConstraintIndex] * AngVelDt - RotationConstraints.Data.ConstraintLambda[ConstraintIndex]) /
		RotationConstraints.Data.ConstraintSoftIM[ConstraintIndex];
	RotationConstraints.Data.ConstraintLambda[ConstraintIndex] += DeltaLambda;

	SolveRotationConstraintDelta(ConstraintIndex, DeltaLambda, false, RotationConstraints);
}

void FPBDJointCachedSolver::ApplyRotationConstraint(
	const int32 ConstraintIndex,
	const FReal Dt)
{
	FReal DeltaAngle = RotationConstraints.Data.ConstraintCX[ConstraintIndex] +
		FVec3::DotProduct(Body(1).DQ() - Body(0).DQ(), RotationConstraints.Data.ConstraintAxis[ConstraintIndex]);

	bool NeedsSolve = false;
	if (RotationConstraints.GetLimitsCheck(ConstraintIndex))
	{
		if (DeltaAngle > RotationConstraints.Data.ConstraintLimits[ConstraintIndex])
		{
			DeltaAngle -= RotationConstraints.Data.ConstraintLimits[ConstraintIndex];
			NeedsSolve = true;
		}
		else if (DeltaAngle < -RotationConstraints.Data.ConstraintLimits[ConstraintIndex])
		{
			DeltaAngle += RotationConstraints.Data.ConstraintLimits[ConstraintIndex];
			NeedsSolve = true;

		}
	}

	if (!RotationConstraints.GetLimitsCheck(ConstraintIndex) || (RotationConstraints.GetLimitsCheck(ConstraintIndex) && NeedsSolve && FMath::Abs(DeltaAngle) > AngleTolerance))
	{
		if (RotationConstraints.GetSoftLimit(ConstraintIndex))
		{
			SolveRotationConstraintSoft(ConstraintIndex, DeltaAngle, Dt, 0.0f);
		}
		else
		{
			SolveRotationConstraintHard(ConstraintIndex, DeltaAngle);
		}
	}
}

void FPBDJointCachedSolver::ApplyRotationSoftConstraintsSimd(
	const FReal Dt)
{
	const VectorRegister4Float Body0DQ = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(Body(0).DQ()[0], Body(0).DQ()[1], Body(0).DQ()[2], 0.0f));
	const VectorRegister4Float Body1DQ = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(Body(1).DQ()[0], Body(1).DQ()[1], Body(1).DQ()[2], 0.0f));

	const VectorRegister4Float DQDiff = VectorSubtract(Body1DQ, Body0DQ);
	VectorRegister4Float ProjAxes[3];
	for (int32 CIndex = 0; CIndex < 3; ++CIndex)
	{
		ProjAxes[CIndex] = Private::VectorDot3FastX(DQDiff, RotationConstraints.Simd.ConstraintAxis[CIndex]);
	}
	VectorRegister4Float ProjAxis = VectorUnpackLo(ProjAxes[0], ProjAxes[1]);
	ProjAxis = VectorMoveLh(ProjAxis, ProjAxes[2]);
	VectorRegister4Float DeltaAngle = VectorAdd(RotationConstraints.Simd.ConstraintCX, ProjAxis);


	const VectorRegister4Float AngleGT = VectorCompareGT(DeltaAngle, RotationConstraints.Simd.ConstraintLimits);
	const VectorRegister4Float AngleLT = VectorCompareLT(DeltaAngle, VectorNegate(RotationConstraints.Simd.ConstraintLimits));

	DeltaAngle = VectorSelect(AngleGT, VectorSubtract(DeltaAngle, RotationConstraints.Simd.ConstraintLimits),
		VectorSelect(AngleLT, VectorAdd(DeltaAngle, RotationConstraints.Simd.ConstraintLimits), DeltaAngle));

	FRealSingle AngleTolerancef = FRealSingle(AngleTolerance);
	const VectorRegister4Float AngleToleranceSimd = VectorLoadFloat1(&AngleTolerancef);
	const VectorRegister4Float InTolerance = VectorCompareGT(VectorAbs(DeltaAngle), AngleToleranceSimd);
	const VectorRegister4Float NeedsSolve = VectorBitwiseAnd(VectorBitwiseOr(AngleGT, AngleLT), InTolerance);

	if (VectorMaskBits(NeedsSolve))
	{
		VectorRegister4Float AngVelDts[3];
		VectorRegister4Float WDiff = VectorSubtract(ConnectorWDtsSimd[0], VectorAdd(DQDiff, ConnectorWDtsSimd[1]));
		for (int32 CIndex = 0; CIndex < 3; ++CIndex)
		{
			AngVelDts[CIndex] = Private::VectorDot3FastX(RotationConstraints.Simd.ConstraintAxis[CIndex], WDiff);
		}
		VectorRegister4Float AngVelDt = VectorUnpackLo(AngVelDts[0], AngVelDts[1]);
		AngVelDt = VectorMoveLh(AngVelDt, AngVelDts[2]);

		const FRealSingle SolverStiffnessf = FRealSingle(SolverStiffness);
		VectorRegister4Float Stiffness = VectorLoadFloat1(&SolverStiffnessf);
		VectorRegister4Float DeltaLambda = VectorMultiply(Stiffness, VectorDivide(VectorSubtract(VectorMultiply(RotationConstraints.Simd.ConstraintSoftStiffness, DeltaAngle),
			VectorAdd(VectorMultiply(RotationConstraints.Simd.ConstraintSoftDamping, AngVelDt), RotationConstraints.Simd.ConstraintLambda)), RotationConstraints.Simd.ConstraintSoftIM));

		DeltaLambda = VectorSelect(NeedsSolve, DeltaLambda, VectorZeroFloat());
		RotationConstraints.Simd.ConstraintLambda = VectorAdd(RotationConstraints.Simd.ConstraintLambda, DeltaLambda);

		VectorRegister4Float DeltaLambdas[3];
		DeltaLambdas[0] = VectorReplicate(DeltaLambda, 0);
		DeltaLambdas[1] = VectorReplicate(DeltaLambda, 1);
		DeltaLambdas[2] = VectorReplicate(DeltaLambda, 2);

		if (Body(0).IsDynamic())
		{
			VectorRegister4Float DR0 = VectorZeroFloat();
			for (int32 CIndex = 0; CIndex < 3; ++CIndex)
			{
				DR0 = VectorMultiplyAdd(RotationConstraints.Simd.ConstraintDRAxis[CIndex][0], DeltaLambdas[CIndex], DR0);
			}
			FVec3f DR0f;
			VectorStoreFloat3(DR0, &DR0f[0]);
			ApplyRotationDelta(0, FVec3(DR0f));
		}
		if (Body(1).IsDynamic())
		{
			VectorRegister4Float DR1 = VectorZeroFloat();
			for (int32 CIndex = 0; CIndex < 3; ++CIndex)
			{
				DR1 = VectorMultiplyAdd(RotationConstraints.Simd.ConstraintDRAxis[CIndex][1], DeltaLambdas[CIndex], DR1);
			}
			FVec3f DR1f;
			VectorStoreFloat3(DR1, &DR1f[0]);
			ApplyRotationDelta(1, FVec3(DR1f));
		}
		NumActiveConstraints += 3;
	}
}


/** APPLY ANGULAR VELOCITY CONSTRAINT *********************************************************************************/

void FPBDJointCachedSolver::ApplyAngularVelocityConstraints()
{
	// SIMD is only enabled when all 3 axes are limited and soft (see InitRotationConstraints).
	// Soft constraints should not apply velocity corrections. They are handled through
	// the damping term in the position solve phase instead. So call to ApplyAngularVelocityConstraintSimd is removed. 

	if (!RotationConstraints.bUseSimd)
	{
		for (int32 ConstraintIndex = 0; ConstraintIndex < 3; ++ConstraintIndex)
		{
			if (RotationConstraints.GetValidDatas(ConstraintIndex) && !RotationConstraints.GetSoftLimit(ConstraintIndex))
			{
				ApplyAngularVelocityConstraint(ConstraintIndex);
			}
		}
	}
}

void FPBDJointCachedSolver::SolveAngularVelocityConstraint(
	const int32 ConstraintIndex,
	const FReal TargetVel)
{
	const FVec3 CW = W(1) - W(0);

	const FReal DeltaLambda = SolverStiffness * RotationConstraints.Data.ConstraintHardStiffness[ConstraintIndex] *
		(FVec3::DotProduct(CW, RotationConstraints.Data.ConstraintAxis[ConstraintIndex]) - TargetVel) / RotationConstraints.Data.ConstraintHardIM[ConstraintIndex];

	// @todo(chaos): we should be adding to the net positional impulse here
	// RotationConstraints.Data.ConstraintLambda[ConsraintIndex] += DeltaLambda * Dt;

	if (Body(0).IsDynamic())
	{
		const FVec3 DW0 = RotationConstraints.Data.ConstraintDRAxis[ConstraintIndex][0] * DeltaLambda;

		Body(0).ApplyAngularVelocityDelta(DW0);
	}
	if (Body(1).IsDynamic())
	{
		const FVec3 DW1 = RotationConstraints.Data.ConstraintDRAxis[ConstraintIndex][1] * DeltaLambda;

		Body(1).ApplyAngularVelocityDelta(DW1);
	}
}

void FPBDJointCachedSolver::ApplyAngularVelocityConstraint(
	const int32 ConstraintIndex)
{
	// Apply restitution for limited joints when we have exceeded the limits
	// We also drive the velocity to zero for locked constraints (ignoring restitution)
	if (FMath::Abs(RotationConstraints.Data.ConstraintLambda[ConstraintIndex]) > UE_SMALL_NUMBER)
	{
		FReal TargetVel = 0.0f;
		if ((RotationConstraints.GetMotionType(ConstraintIndex) == EJointMotionType::Limited) && (RotationConstraints.ConstraintRestitution[ConstraintIndex] != 0.0f))
		{
			const FReal InitVel = InitConstraintAxisAngularVelocities[ConstraintIndex];
			TargetVel = InitVel > Chaos_Joint_AngularVelocityThresholdToApplyRestitution ?
				-RotationConstraints.ConstraintRestitution[ConstraintIndex] * InitVel : 0.0f;
		}
		SolveAngularVelocityConstraint(ConstraintIndex, TargetVel);
	}
}

void FPBDJointCachedSolver::ApplyAngularVelocityConstraintSimd()
{
	/*check(RotationConstraints.MotionType[0] == EJointMotionType::Limited);
	check(RotationConstraints.MotionType[1] == EJointMotionType::Limited);
	check(RotationConstraints.MotionType[2] == EJointMotionType::Limited);*/

	const VectorRegister4Float IsGTEps = VectorCompareGT(VectorAbs(RotationConstraints.Simd.ConstraintLambda), GlobalVectorConstants::SmallNumber);

	if (VectorMaskBits(IsGTEps))
	{
		const VectorRegister4Float Restitution = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(RotationConstraints.ConstraintRestitution[0], RotationConstraints.ConstraintRestitution[1], RotationConstraints.ConstraintRestitution[2], 0.0f));

		const VectorRegister4Float HasRestitution = VectorCompareNE(Restitution, VectorZeroFloat());
		VectorRegister4Float TargetVel = VectorZeroFloat();
		if (VectorMaskBits(HasRestitution))
		{
			const VectorRegister4Float InitVel = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(InitConstraintAxisAngularVelocities[0], InitConstraintAxisAngularVelocities[1], InitConstraintAxisAngularVelocities[2], 0.0f));
			VectorRegister4Float VeclocityThreshold = VectorLoadFloat1(&Chaos_Joint_AngularVelocityThresholdToApplyRestitution);
			TargetVel = VectorSelect(VectorCompareGT(InitVel, VeclocityThreshold), VectorMultiply(VectorNegate(Restitution), InitVel), VectorZeroFloat());
		}

		const FVec3& W0d = W(0);
		const VectorRegister4Float W0 = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(W0d[0], W0d[1], W0d[2], 0.0f));
		const FVec3& W1d = W(1);
		const VectorRegister4Float W1 = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(W1d[0], W1d[1], W1d[2], 0.0f));
		const VectorRegister4Float CW = VectorSubtract(W1, W0);
		const FRealSingle SolverStiffnessf = FRealSingle(SolverStiffness);
		VectorRegister4Float Stiffness = VectorLoadFloat1(&SolverStiffnessf);
		Stiffness = VectorMultiply(Stiffness, RotationConstraints.Simd.ConstraintHardStiffness);

		VectorRegister4Float DeltaLambdas[3];
		for (int32 CIndex = 0; CIndex < 3; CIndex++)
		{
			DeltaLambdas[CIndex] = Private::VectorDot3FastX(CW, RotationConstraints.Simd.ConstraintAxis[CIndex]);
		}
		VectorRegister4Float DeltaLambda = VectorUnpackLo(DeltaLambdas[0], DeltaLambdas[1]);
		DeltaLambda = VectorMoveLh(DeltaLambda, DeltaLambdas[2]);
		DeltaLambda = VectorDivide(VectorMultiply(Stiffness, VectorSubtract(DeltaLambda, TargetVel)), RotationConstraints.Simd.ConstraintHardIM);

		DeltaLambda = VectorSelect(IsGTEps, DeltaLambda, VectorZeroFloat());
		DeltaLambdas[0] = VectorReplicate(DeltaLambda, 0);
		DeltaLambdas[1] = VectorReplicate(DeltaLambda, 1);
		DeltaLambdas[2] = VectorReplicate(DeltaLambda, 2);

		if (Body(0).IsDynamic())
		{
			VectorRegister4Float DW0 = VectorZeroFloat();
			for (int32 CIndex = 0; CIndex < 3; CIndex++)
			{
				DW0 = VectorMultiplyAdd(RotationConstraints.Simd.ConstraintDRAxis[CIndex][0], DeltaLambdas[CIndex], DW0);
			}

			FVec3f DW0f;
			VectorStoreFloat3(DW0, &DW0f[0]);
			Body(0).ApplyAngularVelocityDelta(FVec3(DW0f));
		}
		if (Body(1).IsDynamic())
		{
			VectorRegister4Float DW1 = VectorZeroFloat();
			for (int32 CIndex = 0; CIndex < 3; CIndex++)
			{
				DW1 = VectorMultiplyAdd(RotationConstraints.Simd.ConstraintDRAxis[CIndex][1], DeltaLambdas[CIndex], DW1);
			}
			FVec3f DW1f;
			VectorStoreFloat3(DW1, &DW1f[0]);
			Body(1).ApplyAngularVelocityDelta(FVec3(DW1f));
		}
	}
}

/** INIT POSITION DRIVES *********************************************************************************/

void FPBDJointCachedSolver::InitPositionDrives(
	const FReal Dt,
	const FPBDJointSolverSettings& SolverSettings,
	const FPBDJointSettings& JointSettings)
{
	PositionDrives.SetValidDatas(0, false);
	PositionDrives.SetValidDatas(1, false);
	PositionDrives.SetValidDatas(2, false);
	PositionDrives.bUseSimd = false;

	if (SolverSettings.bEnableDrives)
	{
		TVec3<bool> bDriven =
		{
			(JointSettings.bLinearPositionDriveEnabled[0] || JointSettings.bLinearVelocityDriveEnabled[0]) && (JointSettings.LinearMotionTypes[0] != EJointMotionType::Locked),
			(JointSettings.bLinearPositionDriveEnabled[1] || JointSettings.bLinearVelocityDriveEnabled[1]) && (JointSettings.LinearMotionTypes[1] != EJointMotionType::Locked),
			(JointSettings.bLinearPositionDriveEnabled[2] || JointSettings.bLinearVelocityDriveEnabled[2]) && (JointSettings.LinearMotionTypes[2] != EJointMotionType::Locked),
		};

		PositionDrives.bAccelerationMode = FPBDJointUtilities::GetLinearDriveAccelerationMode(SolverSettings, JointSettings);

		// Rectangular position drives
		if (bDriven[0] || bDriven[1] || bDriven[2])
		{
			const FMatrix33 R0M = ConnectorRs[0].ToMatrix();
			const FVec3 XTarget = ConnectorXs[0] + ConnectorRs[0] * JointSettings.LinearDrivePositionTarget;
			const FVec3 VTarget = ConnectorRs[0] * JointSettings.LinearDriveVelocityTarget;
			const FVec3 CX = ConnectorXs[1] - XTarget;

			const FVec3 ConstraintArm0 = ConnectorXs[1] - CurrentPs[0];
			const FVec3 ConstraintArm1 = ConnectorXs[1] - CurrentPs[1];
			SetInitConstraintVelocity(ConstraintArm0, ConstraintArm1);

			for (int32 AxisIndex = 0; AxisIndex < 3; ++AxisIndex)
			{
				if (bDriven[AxisIndex])
				{
					PositionDrives.InitDatas(AxisIndex, true, FPBDJointUtilities::GetLinearDriveStiffness(SolverSettings, JointSettings, AxisIndex),
						FPBDJointUtilities::GetLinearDriveDamping(SolverSettings, JointSettings, AxisIndex), 0);
					const FVec3 Axis = R0M.GetAxis(AxisIndex);

					if ((FMath::Abs(FVec3::DotProduct(CX, Axis)) > PositionTolerance) || (PositionDrives.Data.ConstraintSoftDamping[AxisIndex] > 0.0f))
					{
						InitAxisPositionDrive(AxisIndex, Axis, CX, VTarget, Dt);
					}

					PositionDrives.SetMaxForce(AxisIndex, JointSettings.LinearDriveMaxForce[AxisIndex], Dt);
				}
			}
		}
	}
}

void FPBDJointCachedSolver::InitAxisPositionDrive(
	const int32 ConstraintIndex,
	const FVec3& ConstraintAxis,
	const FVec3& DeltaPosition,
	const FVec3& DeltaVelocity,
	const FReal Dt)
{
	const FVec3 ConstraintArm0 = ConnectorXs[0] - CurrentPs[0];
	const FVec3 ConstraintArm1 = ConnectorXs[1] - CurrentPs[1];

	PositionDrives.UpdateDatas(ConstraintIndex, ConstraintAxis, FVec3::DotProduct(DeltaPosition, ConstraintAxis),
		0.0f, true, ConstraintArm0, ConstraintArm1,
		FVec3::DotProduct(DeltaVelocity, ConstraintAxis));

	InitPositionDatasMass(PositionDrives, ConstraintIndex, Dt);
}
/** APPLY POSITION PROJECTIONS *********************************************************************************/

void FPBDJointCachedSolver::ApplyProjections(
	const FReal Dt,
	const FPBDJointSolverSettings& SolverSettings,
	const FPBDJointSettings& JointSettings,
	const bool bLastIteration)
{
	if (!JointSettings.bProjectionEnabled)
	{
		return;
	}

	if (!IsDynamic(1))
	{
		// If child is kinematic, return. 
		return;
	}

	SolverStiffness = 1;

	if (SolverSettings.bSolvePositionLast)
	{
		ApplyRotationProjection(Dt, SolverSettings, JointSettings);
		ApplyPositionProjection(Dt, SolverSettings, JointSettings);
	}
	else
	{
		ApplyPositionProjection(Dt, SolverSettings, JointSettings);
		ApplyRotationProjection(Dt, SolverSettings, JointSettings);
	}

	if (bLastIteration)
	{
		// Add velocity correction from the net projection motion
		// @todo(chaos): this should be a joint setting?
		if (Chaos_Joint_VelProjectionAlpha > 0.0f)
		{
			const FSolverReal VelocityScale = Chaos_Joint_VelProjectionAlpha / static_cast<FSolverReal>(Dt);
			const FSolverVec3 DV1 = Body1().DP() * VelocityScale;
			const FSolverVec3 DW1 = Body1().DQ() * VelocityScale;

			Body(1).ApplyVelocityDelta(DV1, DW1);
		}
	}
}

void FPBDJointCachedSolver::ApplyRotationProjection(
	const FReal Dt,
	const FPBDJointSolverSettings& SolverSettings,
	const FPBDJointSettings& JointSettings)
{
	const FReal AngularProjection = FPBDJointUtilities::GetAngularProjection(SolverSettings, JointSettings);
	if (AngularProjection == 0)
	{
		return;
	}
	const TVec3<EJointMotionType>& LinearMotion = JointSettings.LinearMotionTypes;
	const bool bLinearLocked = (LinearMotion[0] == EJointMotionType::Locked) && (LinearMotion[1] == EJointMotionType::Locked) && (LinearMotion[2] == EJointMotionType::Locked);
	if (RotationConstraints.bUseSimd)
	{
		// TODO Here there is a paradox it can be vectorized only if soft but cannot be projected if soft 
		ApplyRotationProjectionSimd(FRealSingle(AngularProjection), bLinearLocked);
	}
	else
	{
		for (int32 ConstraintIndex = 0; ConstraintIndex < 3; ++ConstraintIndex)
		{
			ApplyAxisRotationProjection(AngularProjection, bLinearLocked, ConstraintIndex);
		}
	}
}

void FPBDJointCachedSolver::ApplyAxisRotationProjection(
	const FReal AngularProjection,
	const bool bLinearLocked,
	const int32 ConstraintIndex)
{
	check(RotationConstraints.bUseSimd == false);
	if (RotationConstraints.GetValidDatas(ConstraintIndex) && !RotationConstraints.GetSoftLimit(ConstraintIndex))
	{
		FReal DeltaAngle = RotationConstraints.Data.ConstraintCX[ConstraintIndex] +
			FVec3::DotProduct(Body(1).DQ() - Body(0).DQ(), RotationConstraints.Data.ConstraintAxis[ConstraintIndex]);

		bool NeedsSolve = false;
		if (RotationConstraints.GetLimitsCheck(ConstraintIndex))
		{
			if (DeltaAngle > RotationConstraints.Data.ConstraintLimits[ConstraintIndex])
			{
				DeltaAngle -= RotationConstraints.Data.ConstraintLimits[ConstraintIndex];
				NeedsSolve = true;
			}
			else if (DeltaAngle < -RotationConstraints.Data.ConstraintLimits[ConstraintIndex])
			{
				DeltaAngle += RotationConstraints.Data.ConstraintLimits[ConstraintIndex];
				NeedsSolve = true;

			}
		}

		if (!RotationConstraints.GetLimitsCheck(ConstraintIndex) || (RotationConstraints.GetLimitsCheck(ConstraintIndex) && NeedsSolve && FMath::Abs(DeltaAngle) > AngleTolerance))
		{
			const FReal IM = -FVec3::DotProduct(RotationConstraints.Data.ConstraintAxis[ConstraintIndex], RotationConstraints.Data.ConstraintDRAxis[ConstraintIndex][1]);
			const FReal DeltaLambda = SolverStiffness * RotationConstraints.Data.ConstraintHardStiffness[ConstraintIndex] * DeltaAngle / IM;

			const FVec3 DR1 = AngularProjection * RotationConstraints.Data.ConstraintDRAxis[ConstraintIndex][1] * DeltaLambda;
			ApplyRotationDelta(1, DR1);

			if (bLinearLocked)
			{
				FVec3 PositionConstraintsArms1;
				if (!PositionConstraints.bUseSimd)
				{
					PositionConstraintsArms1 = PositionConstraints.Data.ConstraintArms[ConstraintIndex][1];
				}
				else
				{
					FVec3f PositionConstraintsArmsf;
					VectorStoreFloat3(PositionConstraints.Simd.ConstraintArms[1], &PositionConstraintsArmsf[0]);
					PositionConstraintsArms1 = FVec3(PositionConstraintsArmsf);
				}
				const FVec3 DP1 = -AngularProjection * FVec3::CrossProduct(DR1, PositionConstraintsArms1);
				ApplyPositionDelta(1, DP1);
			}
		}
	}
}

void FPBDJointCachedSolver::ApplyRotationProjectionSimd(
	const FRealSingle AngularProjectionf,
	const bool bLinearLocked)
{
	const VectorRegister4Float Body0DQ = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(Body(0).DQ()[0], Body(0).DQ()[1], Body(0).DQ()[2], 0.0f));
	const VectorRegister4Float Body1DQ = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(Body(1).DQ()[0], Body(1).DQ()[1], Body(1).DQ()[2], 0.0f));

	const VectorRegister4Float DQDiff = VectorSubtract(Body1DQ, Body0DQ);
	VectorRegister4Float ProjAxes[3];
	for (int32 CIndex = 0; CIndex < 3; ++CIndex)
	{
		ProjAxes[CIndex] = Private::VectorDot3FastX(DQDiff, RotationConstraints.Simd.ConstraintAxis[CIndex]);
	}
	VectorRegister4Float ProjAxis = VectorUnpackLo(ProjAxes[0], ProjAxes[1]);
	ProjAxis = VectorMoveLh(ProjAxis, ProjAxes[2]);
	VectorRegister4Float DeltaAngle = VectorAdd(RotationConstraints.Simd.ConstraintCX, ProjAxis);


	const VectorRegister4Float AngleGT = VectorCompareGT(DeltaAngle, RotationConstraints.Simd.ConstraintLimits);
	const VectorRegister4Float AngleLT = VectorCompareLT(DeltaAngle, VectorNegate(RotationConstraints.Simd.ConstraintLimits));

	DeltaAngle = VectorSelect(AngleGT, VectorSubtract(DeltaAngle, RotationConstraints.Simd.ConstraintLimits),
		VectorSelect(AngleLT, VectorAdd(DeltaAngle, RotationConstraints.Simd.ConstraintLimits), DeltaAngle));

	FRealSingle AngleTolerancef = FRealSingle(AngleTolerance);
	const VectorRegister4Float AngleToleranceSimd = VectorLoadFloat1(&AngleTolerancef);
	const VectorRegister4Float InTolerance = VectorCompareGT(VectorAbs(DeltaAngle), AngleToleranceSimd);
	const VectorRegister4Float NeedsSolve = VectorBitwiseAnd(VectorBitwiseOr(AngleGT, AngleLT), InTolerance);

	if (VectorMaskBits(NeedsSolve))
	{
		VectorRegister4Float IMs[3];
		for (int32 CIndex = 0; CIndex < 3; ++CIndex)
		{
			IMs[CIndex] = Private::VectorDot3FastX(RotationConstraints.Simd.ConstraintAxis[CIndex], RotationConstraints.Simd.ConstraintDRAxis[CIndex][1]);
		}
		VectorRegister4Float IM = VectorUnpackLo(IMs[0], IMs[1]);
		IM = VectorMoveLh(IM, IMs[2]);
		IM = VectorNegate(IM);

		const FRealSingle SolverStiffnessf = FRealSingle(SolverStiffness);
		VectorRegister4Float Stiffness = VectorLoadFloat1(&SolverStiffnessf);
		VectorRegister4Float DeltaLambda = VectorDivide(VectorMultiply(VectorMultiply(Stiffness, RotationConstraints.Simd.ConstraintHardStiffness), DeltaAngle), IM);
		DeltaLambda = VectorSelect(NeedsSolve, DeltaLambda, VectorZeroFloat());

		VectorRegister4Float AngularProjection = VectorLoadFloat1(&AngularProjectionf);
		DeltaLambda = VectorMultiply(DeltaLambda, AngularProjection);

		VectorRegister4Float DeltaLambdas[3];
		DeltaLambdas[0] = VectorReplicate(DeltaLambda, 0);
		DeltaLambdas[1] = VectorReplicate(DeltaLambda, 1);
		DeltaLambdas[2] = VectorReplicate(DeltaLambda, 2);

		VectorRegister4Float DR1 = VectorZeroFloat();
		VectorRegister4Float DR1s[3];
		for (int32 CIndex = 0; CIndex < 3; ++CIndex)
		{
			DR1s[CIndex] = VectorMultiply(RotationConstraints.Simd.ConstraintDRAxis[CIndex][1], DeltaLambdas[CIndex]);
			DR1 = VectorAdd(DR1s[CIndex], DR1);
		}
		FVec3f DR1f;
		VectorStoreFloat3(DR1, &DR1f[0]);
		ApplyRotationDelta(1, FVec3(DR1f));

		if (bLinearLocked)
		{
			VectorRegister4Float DP1 = VectorZeroFloat();
			AngularProjection = VectorNegate(AngularProjection);
			for (int32 CIndex = 0; CIndex < 3; ++CIndex)
			{
				VectorRegister4Float PositionConstraintsArms1;
				if (PositionConstraints.bUseSimd)
				{
					PositionConstraintsArms1 = PositionConstraints.Simd.ConstraintArms[1];
				}
				else
				{
					PositionConstraintsArms1 = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(PositionConstraints.Data.ConstraintArms[CIndex][1][0], PositionConstraints.Data.ConstraintArms[CIndex][1][1], PositionConstraints.Data.ConstraintArms[CIndex][1][2], 0.0));
				}
				DP1 = VectorMultiplyAdd(AngularProjection, VectorCross(DR1, PositionConstraintsArms1), DP1);
			}

			FVec3f DP1f;
			VectorStoreFloat3(DP1, &DP1f[0]);
			ApplyPositionDelta(1, FVec3(DP1f));
		}
	}
}

void FPBDJointCachedSolver::ApplyPositionProjection(
	const FReal Dt,
	const FPBDJointSolverSettings& SolverSettings,
	const FPBDJointSettings& JointSettings)
{
	const FReal LinearProjection = FPBDJointUtilities::GetLinearProjection(SolverSettings, JointSettings);
	if (LinearProjection == 0)
	{
		return;
	}

	if (PositionConstraints.bUseSimd)
	{
		ApplyPositionProjectionSimd(LinearProjection);
	}
	else
	{
		for (int32 ConstraintIndex = 0; ConstraintIndex < 3; ++ConstraintIndex)
		{
			ApplyAxisPositionProjection(LinearProjection, ConstraintIndex);
		}
	}
}

void FPBDJointCachedSolver::ApplyAxisPositionProjection(
	const FReal LinearProjection,
	const int32 ConstraintIndex)
{
	if (PositionConstraints.GetValidDatas(ConstraintIndex) && !PositionConstraints.GetSoftLimit(ConstraintIndex))
	{
		const FVec3 CX = Body(1).DP() - Body(0).DP() +
			FVec3::CrossProduct(Body(1).DQ(), PositionConstraints.Data.ConstraintArms[ConstraintIndex][1]) -
			FVec3::CrossProduct(Body(0).DQ(), PositionConstraints.Data.ConstraintArms[ConstraintIndex][0]);

		FReal DeltaPosition = PositionConstraints.Data.ConstraintCX[ConstraintIndex] + FVec3::DotProduct(CX, PositionConstraints.Data.ConstraintAxis[ConstraintIndex]);

		bool NeedsSolve = false;
		if (PositionConstraints.GetLimitsCheck(ConstraintIndex))
		{
			if (DeltaPosition > PositionConstraints.Data.ConstraintLimits[ConstraintIndex])
			{
				DeltaPosition -= PositionConstraints.Data.ConstraintLimits[ConstraintIndex];
				NeedsSolve = true;
			}
			else if (DeltaPosition < -PositionConstraints.Data.ConstraintLimits[ConstraintIndex])
			{
				DeltaPosition += PositionConstraints.Data.ConstraintLimits[ConstraintIndex];
				NeedsSolve = true;
			}
		}
		if (!PositionConstraints.GetLimitsCheck(ConstraintIndex) || (PositionConstraints.GetLimitsCheck(ConstraintIndex) && NeedsSolve && FMath::Abs(DeltaPosition) > PositionTolerance))
		{
			const FVec3 AngularAxis1 = FVec3::CrossProduct(PositionConstraints.Data.ConstraintArms[ConstraintIndex][1], PositionConstraints.Data.ConstraintAxis[ConstraintIndex]);
			const FReal IM = InvM(1) - FVec3::DotProduct(AngularAxis1, PositionConstraints.Data.ConstraintDRAxis[ConstraintIndex][1]);
			const FReal DeltaLambda = SolverStiffness * PositionConstraints.Data.ConstraintHardStiffness[ConstraintIndex] * DeltaPosition / IM;

			const FVec3 DX = PositionConstraints.Data.ConstraintAxis[ConstraintIndex] * DeltaLambda;

			const FVec3 DP1 = -LinearProjection * InvM(1) * DX;
			const FVec3 DR1 = LinearProjection * PositionConstraints.Data.ConstraintDRAxis[ConstraintIndex][1] * DeltaLambda;

			ApplyPositionDelta(1, DP1);
			ApplyRotationDelta(1, DR1);
		}
	}
}

void FPBDJointCachedSolver::ApplyPositionProjectionSimd(
	const FReal LinearProjection)
{
	const VectorRegister4Float Body0DP = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(Body(0).DP()[0], Body(0).DP()[1], Body(0).DP()[2], 0.0f));
	const VectorRegister4Float Body1DP = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(Body(1).DP()[0], Body(1).DP()[1], Body(1).DP()[2], 0.0f));

	const VectorRegister4Float Body0DQ = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(Body(0).DQ()[0], Body(0).DQ()[1], Body(0).DQ()[2], 0.0f));
	const VectorRegister4Float Body1DQ = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(Body(1).DQ()[0], Body(1).DQ()[1], Body(1).DQ()[2], 0.0f));

	const VectorRegister4Float DPDiff = VectorSubtract(Body1DP, Body0DP);

	const VectorRegister4Float Cross1 = VectorCross(Body1DQ, PositionConstraints.Simd.ConstraintArms[1]);
	const VectorRegister4Float Cross0 = VectorCross(Body0DQ, PositionConstraints.Simd.ConstraintArms[0]);
	const VectorRegister4Float CrossDiff = VectorSubtract(Cross1, Cross0);
	const VectorRegister4Float CX = VectorAdd(DPDiff, CrossDiff);

	VectorRegister4Float DeltaPositions[3];
	for (int32 CIndex = 0; CIndex < 3; ++CIndex)
	{
		DeltaPositions[CIndex] = Private::VectorDot3FastX(CX, PositionConstraints.Simd.ConstraintAxis[CIndex]);
	}
	VectorRegister4Float DeltaPosition = VectorUnpackLo(DeltaPositions[0], DeltaPositions[1]);
	DeltaPosition = VectorMoveLh(DeltaPosition, DeltaPositions[2]);
	DeltaPosition = VectorAdd(DeltaPosition, PositionConstraints.Simd.ConstraintCX);

	const FRealSingle SolverStiffnessf = FRealSingle(SolverStiffness);
	VectorRegister4Float Stiffness = VectorLoadFloat1(&SolverStiffnessf);

	Stiffness = VectorMultiply(Stiffness, PositionConstraints.Simd.ConstraintHardStiffness);
	DeltaPosition = VectorMultiply(Stiffness, DeltaPosition);

	FRealSingle InvM1f = FRealSingle(InvM(1));
	const VectorRegister4Float InvM1 = VectorLoadFloat1(&InvM1f);

	VectorRegister4Float IMs[3];
	for (int32 CIndex = 0; CIndex < 3; CIndex++)
	{
		const VectorRegister4Float AngularAxis1 = VectorCross(PositionConstraints.Simd.ConstraintArms[1], PositionConstraints.Simd.ConstraintAxis[CIndex]);
		IMs[CIndex] = Private::VectorDot3FastX(AngularAxis1, PositionConstraints.Simd.ConstraintDRAxis[CIndex][1]);
	}
	VectorRegister4Float IM = VectorUnpackLo(IMs[0], IMs[1]);
	IM = VectorMoveLh(IM, IMs[2]);
	IM = VectorSubtract(InvM1, IM);

	const VectorRegister4Float DeltaLambda = VectorDivide(DeltaPosition, IM);
	VectorRegister4Float DeltaLambdas[3];
	DeltaLambdas[0] = VectorReplicate(DeltaLambda, 0);
	DeltaLambdas[1] = VectorReplicate(DeltaLambda, 1);
	DeltaLambdas[2] = VectorReplicate(DeltaLambda, 2);

	FRealSingle NegLinPorjInvMf = -FRealSingle(LinearProjection * InvM(1));
	VectorRegister4Float NegLinPorjInvM = VectorLoadFloat1(&NegLinPorjInvMf);
	FRealSingle LinearProjectionf = FRealSingle(LinearProjection);
	VectorRegister4Float LinearProjectionSimd = VectorLoadFloat1(&LinearProjectionf);

	VectorRegister4Float DP1 = VectorZeroFloat();
	VectorRegister4Float DR1 = VectorZeroFloat();
	for (int32 CIndex = 0; CIndex < 3; CIndex++)
	{
		VectorRegister4Float DX = VectorMultiply(PositionConstraints.Simd.ConstraintAxis[CIndex], DeltaLambdas[CIndex]);
		DP1 = VectorMultiplyAdd(NegLinPorjInvM, DX, DP1);
		DR1 = VectorMultiplyAdd(VectorMultiply(LinearProjectionSimd, PositionConstraints.Simd.ConstraintDRAxis[CIndex][1]), DeltaLambdas[CIndex], DR1);
	}
	FVec3f DP1f;
	VectorStoreFloat3(DP1, &DP1f[0]);
	ApplyPositionDelta(1, FVec3(DP1f));
	FVec3f DR1f;
	VectorStoreFloat3(DR1, &DR1f[0]);
	ApplyRotationDelta(1, FVec3(DR1f));

}

void FPBDJointCachedSolver::ApplyTeleports(
	const FReal Dt,
	const FPBDJointSolverSettings& SolverSettings,
	const FPBDJointSettings& JointSettings)
{
	ApplyRotationTeleport(Dt, SolverSettings, JointSettings);
	ApplyPositionTeleport(Dt, SolverSettings, JointSettings);
}


void FPBDJointCachedSolver::ApplyPositionTeleport(
	const FReal Dt,
	const FPBDJointSolverSettings& SolverSettings,
	const FPBDJointSettings& JointSettings)
{
	if (JointSettings.TeleportDistance <= 0)
	{
		return;
	}
	if (PositionConstraints.bUseSimd)
	{
		ApplyPositionTeleportSimd(FRealSingle(JointSettings.TeleportDistance));
	}
	else
	{
		for (int32 ConstraintIndex = 0; ConstraintIndex < 3; ++ConstraintIndex)
		{
			ApplyAxisPositionTeleport(JointSettings.TeleportDistance, ConstraintIndex);
		}
	}
}

void FPBDJointCachedSolver::ApplyAxisPositionTeleport(
	const FReal TeleportDistance,
	const int32 ConstraintIndex)
{
	check(PositionConstraints.bUseSimd == false);

	if (PositionConstraints.GetValidDatas(ConstraintIndex) && !PositionConstraints.GetSoftLimit(ConstraintIndex))
	{
		FReal DeltaPosition = PositionConstraints.Data.ConstraintCX[ConstraintIndex];

		bool NeedsSolve = false;
		if (PositionConstraints.GetLimitsCheck(ConstraintIndex))
		{
			if (DeltaPosition > PositionConstraints.Data.ConstraintLimits[ConstraintIndex])
			{
				DeltaPosition -= PositionConstraints.Data.ConstraintLimits[ConstraintIndex];
				NeedsSolve = true;
			}
			else if (DeltaPosition < -PositionConstraints.Data.ConstraintLimits[ConstraintIndex])
			{
				DeltaPosition += PositionConstraints.Data.ConstraintLimits[ConstraintIndex];
				NeedsSolve = true;
			}
		}
		if (!PositionConstraints.GetLimitsCheck(ConstraintIndex) || (PositionConstraints.GetLimitsCheck(ConstraintIndex) && NeedsSolve))
		{
			if (FMath::Abs(DeltaPosition) > TeleportDistance)
			{
				const FVec3 DP1 = -DeltaPosition * PositionConstraints.Data.ConstraintAxis[ConstraintIndex];
				ApplyPositionDelta(1, DP1);
			}
		}
	}
}

void FPBDJointCachedSolver::ApplyPositionTeleportSimd(
	const FRealSingle TeleportDistance)
{
	const VectorRegister4Float TeleportDistanceSimd = VectorLoadFloat1(&TeleportDistance);
	const VectorRegister4Float IsGT = VectorCompareGT(VectorAbs(PositionConstraints.Simd.ConstraintCX), TeleportDistanceSimd);

	if (VectorMaskBits(IsGT))
	{
		const VectorRegister4Float ConstraintCX = VectorSelect(IsGT, VectorNegate(PositionConstraints.Simd.ConstraintCX), VectorZeroFloat());

		VectorRegister4Float ConstraintCXs[3];
		ConstraintCXs[0] = VectorReplicate(ConstraintCX, 0);
		ConstraintCXs[1] = VectorReplicate(ConstraintCX, 1);
		ConstraintCXs[2] = VectorReplicate(ConstraintCX, 2);
		VectorRegister4Float DP1 = VectorMultiply(PositionConstraints.Simd.ConstraintAxis[0], ConstraintCXs[0]);
		for (int32 ConstraintIndex = 1; ConstraintIndex < 3; ++ConstraintIndex)
		{
			DP1 = VectorMultiplyAdd(PositionConstraints.Simd.ConstraintAxis[ConstraintIndex], ConstraintCXs[ConstraintIndex], DP1);
		}
		FVec3f DP1f;
		VectorStoreFloat3(DP1, &DP1f[0]);
		ApplyPositionDelta(1, FVec3(DP1f));
	}
}

void FPBDJointCachedSolver::ApplyRotationTeleport(
	const FReal Dt,
	const FPBDJointSolverSettings& SolverSettings,
	const FPBDJointSettings& JointSettings)
{
	if (JointSettings.TeleportAngle <= 0)
	{
		return;
	}
}


/** APPLY POSITION  DRIVES *********************************************************************************/

void FPBDJointCachedSolver::ApplyPositionDrives(
	const FReal Dt)
{
	for (int32 ConstraintIndex = 0; ConstraintIndex < 3; ++ConstraintIndex)
	{
		if (PositionDrives.GetValidDatas(ConstraintIndex))
		{
			ApplyAxisPositionDrive(ConstraintIndex, Dt);
		}
	}
}


void FPBDJointCachedSolver::ApplyAxisPositionDrive(
	const int32 ConstraintIndex,
	const FReal Dt)
{
	check(PositionDrives.bUseSimd == false);
	const FReal Stiffness = PositionDrives.Data.ConstraintSoftStiffness[ConstraintIndex];
	const FReal Damping = PositionDrives.Data.ConstraintSoftDamping[ConstraintIndex];
	const FReal IM = PositionDrives.Data.ConstraintSoftIM[ConstraintIndex];

	const FVec3 Delta0 = Body(0).DP() + FVec3::CrossProduct(Body(0).DQ(), PositionDrives.Data.ConstraintArms[ConstraintIndex][0]);
	const FVec3 Delta1 = Body(1).DP() + FVec3::CrossProduct(Body(1).DQ(), PositionDrives.Data.ConstraintArms[ConstraintIndex][1]);
	const FReal CX = PositionDrives.Data.ConstraintCX[ConstraintIndex] + FVec3::DotProduct(Delta1 - Delta0, PositionDrives.Data.ConstraintAxis[ConstraintIndex]);

	FReal CVDt = 0;
	if (Damping > UE_KINDA_SMALL_NUMBER)
	{
		const FVec3 V0Dt = FVec3::CalculateVelocity(InitConnectorXs[0], ConnectorXs[0] + Delta0, 1.0f);
		const FVec3 V1Dt = FVec3::CalculateVelocity(InitConnectorXs[1], ConnectorXs[1] + Delta1, 1.0f);
		const FReal TargetVDt = PositionDrives.ConstraintVX[ConstraintIndex] * Dt;
		CVDt = TargetVDt + FVec3::DotProduct(V0Dt - V1Dt, PositionDrives.Data.ConstraintAxis[ConstraintIndex]);
	}

	FReal Lambda = PositionDrives.Data.ConstraintLambda[ConstraintIndex];
	FReal DeltaLambda = SolverStiffness * (Stiffness * CX - Damping * CVDt - Lambda) / IM;
	Lambda += DeltaLambda;

	PositionDrives.ApplyMaxLambda(ConstraintIndex, DeltaLambda, Lambda);
	PositionDrives.Data.ConstraintLambda[ConstraintIndex] = Lambda;

	SolvePositionConstraintDelta(ConstraintIndex, DeltaLambda, PositionDrives);
}

void FPBDJointCachedSolver::ApplyPositionVelocityDrives(
	const FReal Dt)
{
	if (bUsePositionBasedDrives)
	{
		return;
	}

	for (int32 ConstraintIndex = 0; ConstraintIndex < 3; ++ConstraintIndex)
	{
		if (PositionDrives.GetValidDatas(ConstraintIndex))
		{
			ApplyAxisPositionVelocityDrive(ConstraintIndex, Dt);
		}
	}
}

void FPBDJointCachedSolver::ApplyAxisPositionVelocityDrive(
	const int32 ConstraintIndex,
	const FReal Dt)
{
	check(PositionDrives.bUseSimd == false);
	// NOTE: Using the actual damping, not the PBD modified value
	const FReal Damping = PositionDrives.SettingsSoftDamping[ConstraintIndex] * Dt;
	if (Damping < UE_SMALL_NUMBER)
	{
		return;
	}

	const FReal MassScale = PositionDrives.bAccelerationMode ? (FReal(1) / PositionDrives.Data.ConstraintHardIM[ConstraintIndex]) : FReal(1);
	const FReal IM = MassScale * Damping * PositionDrives.Data.ConstraintHardIM[ConstraintIndex] + (FReal)1;

	// Velocity error to correct
	const FVec3 V0 = V(0) + FVec3::CrossProduct(W(0), PositionDrives.Data.ConstraintArms[ConstraintIndex][0]);
	const FVec3 V1 = V(1) + FVec3::CrossProduct(W(1), PositionDrives.Data.ConstraintArms[ConstraintIndex][1]);
	const FReal VRel = FVec3::DotProduct(V1 - V0, PositionDrives.Data.ConstraintAxis[ConstraintIndex]);
	const FReal TargetV = PositionDrives.ConstraintVX[ConstraintIndex];
	const FReal CV = (VRel - TargetV);

	// Implicit scheme: F(t) = -D x V(t+dt)
	FReal& LambdaVel = PositionDrives.ConstraintLambdaVelocity[ConstraintIndex];
	FReal DeltaLambdaVel = SolverStiffness * (MassScale * Damping * CV - LambdaVel) / IM;

	// Apply limits and accumulate total impulse 
	// (NOTE: Limits and net impulses are position based, not velocity based)
	FReal DeltaLambda = DeltaLambdaVel * Dt;
	FReal Lambda = PositionDrives.Data.ConstraintLambda[ConstraintIndex] + DeltaLambda;
	PositionDrives.ApplyMaxLambda(ConstraintIndex, DeltaLambda, Lambda);
	PositionDrives.Data.ConstraintLambda[ConstraintIndex] = Lambda;
	DeltaLambdaVel = DeltaLambda / Dt;

	LambdaVel += DeltaLambdaVel;
	const FVec3 Impulse = DeltaLambdaVel * PositionDrives.Data.ConstraintAxis[ConstraintIndex];

	if (Body(0).IsDynamic())
	{
		const FVec3 DV0 = InvM(0) * Impulse;
		const FVec3 DW0 = PositionDrives.Data.ConstraintDRAxis[ConstraintIndex][0] * DeltaLambdaVel;

		Body(0).ApplyVelocityDelta(DV0, DW0);
	}
	if (Body(1).IsDynamic())
	{
		const FVec3 DV1 = -InvM(1) * Impulse;
		const FVec3 DW1 = PositionDrives.Data.ConstraintDRAxis[ConstraintIndex][1] * DeltaLambdaVel;

		Body(1).ApplyVelocityDelta(DV1, DW1);
	}
}


/** INIT ROTATION DRIVES *********************************************************************************/

void FPBDJointCachedSolver::InitRotationDrives(
	const FReal Dt,
	const FPBDJointSolverSettings& SolverSettings,
	const FPBDJointSettings& JointSettings)
{
	RotationDrives.SetValidDatas(0, false);
	RotationDrives.SetValidDatas(1, false);
	RotationDrives.SetValidDatas(2, false);
	RotationDrives.bUseSimd = false;

	bool bHasRotationDrives =
		JointSettings.bAngularTwistPositionDriveEnabled
		|| JointSettings.bAngularTwistVelocityDriveEnabled
		|| JointSettings.bAngularSwingPositionDriveEnabled
		|| JointSettings.bAngularSwingVelocityDriveEnabled
		|| JointSettings.bAngularSLerpPositionDriveEnabled
		|| JointSettings.bAngularSLerpVelocityDriveEnabled;
	if (!bHasRotationDrives)
	{
		return;
	}

	if (SolverSettings.bEnableDrives)
	{
		EJointMotionType TwistMotion = JointSettings.AngularMotionTypes[(int32)EJointAngularConstraintIndex::Twist];
		EJointMotionType Swing1Motion = JointSettings.AngularMotionTypes[(int32)EJointAngularConstraintIndex::Swing1];
		EJointMotionType Swing2Motion = JointSettings.AngularMotionTypes[(int32)EJointAngularConstraintIndex::Swing2];
		bool bTwistLocked = TwistMotion == EJointMotionType::Locked;
		bool bSwing1Locked = Swing1Motion == EJointMotionType::Locked;
		bool bSwing2Locked = Swing2Motion == EJointMotionType::Locked;

		// No SLerp drive if we have a locked rotation (it will be grayed out in the editor in this case, but could still have been set before the rotation was locked)
		// @todo(ccaulfield): setting should be cleaned up before being passed to the solver
		if ((JointSettings.bAngularSLerpPositionDriveEnabled || JointSettings.bAngularSLerpVelocityDriveEnabled) && !bTwistLocked && !bSwing1Locked && !bSwing2Locked)
		{
			RotationDrives.bUseSimd = bUseSimd;
			InitSLerpDrive(Dt, SolverSettings, JointSettings);
		}
		else
		{
			const bool bTwistDriveEnabled = ((JointSettings.bAngularTwistPositionDriveEnabled || JointSettings.bAngularTwistVelocityDriveEnabled) && !bTwistLocked);
			const bool bSwingDriveEnabled = (JointSettings.bAngularSwingPositionDriveEnabled || JointSettings.bAngularSwingVelocityDriveEnabled);
			const bool bSwing1DriveEnabled = bSwingDriveEnabled && !bSwing1Locked;
			const bool bSwing2DriveEnabled = bSwingDriveEnabled && !bSwing2Locked;
			if (bTwistDriveEnabled || bSwing1DriveEnabled || bSwing2DriveEnabled)
			{
				InitSwingTwistDrives(Dt, SolverSettings, JointSettings, bTwistDriveEnabled, bSwing1DriveEnabled, bSwing2DriveEnabled);
			}
		}
	}
}

void FPBDJointCachedSolver::InitRotationConstraintDrive(
	const int32 ConstraintIndex,
	const FVec3& ConstraintAxis,
	const FReal Dt,
	const FReal DeltaAngle)
{
	RotationDrives.UpdateDatas(ConstraintIndex, ConstraintAxis, DeltaAngle, 0.0f);

	InitRotationDatasMass(RotationDrives, ConstraintIndex, Dt);
}

void FPBDJointCachedSolver::InitRotationConstraintDriveSimd(
	FVec3 ConstraintAxes[3],
	const FRealSingle Dtf,
	const FVec3 DeltaAngles
)
{
	FVec3 ConstraintHardIM;
	RotationDrives.Simd.ConstraintCX = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(DeltaAngles[0], DeltaAngles[1], DeltaAngles[2], 0.0f));

	for (int32 ConstraintIndex = 0; ConstraintIndex < 3; ConstraintIndex++)
	{
		RotationDrives.ConstraintVX[ConstraintIndex] = 0.0;

		RotationDrives.Simd.ConstraintAxis[ConstraintIndex] = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(ConstraintAxes[ConstraintIndex][0], ConstraintAxes[ConstraintIndex][1], ConstraintAxes[ConstraintIndex][2], 0.0f));

		const VectorRegister4Float& Axis = RotationDrives.Simd.ConstraintAxis[ConstraintIndex];
		const VectorRegister4Float AxisX = VectorReplicate(Axis, 0);
		const VectorRegister4Float AxisY = VectorReplicate(Axis, 1);
		const VectorRegister4Float AxisZ = VectorReplicate(Axis, 2);
		const FMatrix33& InvI0 = InvI(0);
		const VectorRegister4Float InvI00 = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(InvI0.M[0][0], InvI0.M[0][1], InvI0.M[0][2], 0.0f));
		const VectorRegister4Float InvI01 = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(InvI0.M[1][0], InvI0.M[1][1], InvI0.M[1][2], 0.0f));
		const VectorRegister4Float InvI02 = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(InvI0.M[2][0], InvI0.M[2][1], InvI0.M[2][2], 0.0f));
		const VectorRegister4Float IA0 = VectorMultiplyAdd(InvI00, AxisX, VectorMultiplyAdd(InvI01, AxisY, VectorMultiply(InvI02, AxisZ)));

		const FMatrix33& InvI1 = InvI(1);
		const VectorRegister4Float InvI10 = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(InvI1.M[0][0], InvI1.M[0][1], InvI1.M[0][2], 0.0f));
		const VectorRegister4Float InvI11 = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(InvI1.M[1][0], InvI1.M[1][1], InvI1.M[1][2], 0.0f));
		const VectorRegister4Float InvI12 = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(InvI1.M[2][0], InvI1.M[2][1], InvI1.M[2][2], 0.0f));
		const VectorRegister4Float IA1 = VectorMultiplyAdd(InvI10, AxisX, VectorMultiplyAdd(InvI11, AxisY, VectorMultiply(InvI12, AxisZ)));

		const FRealSingle II0 = VectorDot3Scalar(Axis, IA0);
		const FRealSingle II1 = VectorDot3Scalar(Axis, IA1);

		ConstraintHardIM[ConstraintIndex] = II0 + II1;

		RotationDrives.Simd.ConstraintDRAxis[ConstraintIndex][0] = IA0;
		RotationDrives.Simd.ConstraintDRAxis[ConstraintIndex][1] = VectorNegate(IA1);

		check(RotationDrives.GetSoftLimit(ConstraintIndex));
	}
	RotationDrives.Simd.ConstraintHardIM = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(ConstraintHardIM[0], ConstraintHardIM[1], ConstraintHardIM[2], 0.0f));
	const VectorRegister4Float SoftDamping = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(RotationDrives.SettingsSoftDamping[0], RotationDrives.SettingsSoftDamping[1], RotationDrives.SettingsSoftDamping[2], 0.0f));
	const VectorRegister4Float SoftStiffness = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(RotationDrives.SettingsSoftStiffness[0], RotationDrives.SettingsSoftStiffness[1], RotationDrives.SettingsSoftStiffness[2], 0.0f));
	const VectorRegister4Float Dt = VectorLoadFloat1(&Dtf);

	const VectorRegister4Float SpringMassScale = RotationDrives.bAccelerationMode ? VectorDivide(GlobalVectorConstants::FloatOne, RotationDrives.Simd.ConstraintHardIM) : GlobalVectorConstants::FloatOne;
	RotationDrives.Simd.ConstraintSoftStiffness = VectorMultiply(VectorMultiply(SpringMassScale, SoftStiffness), VectorMultiply(Dt, Dt));
	RotationDrives.Simd.ConstraintSoftDamping = bUsePositionBasedDrives ? VectorMultiply(SpringMassScale, VectorMultiply(SoftDamping, Dt)) : VectorZeroFloat();
	RotationDrives.Simd.ConstraintSoftIM = VectorAdd(VectorMultiply(VectorAdd(RotationDrives.Simd.ConstraintSoftStiffness, RotationDrives.Simd.ConstraintSoftDamping), RotationDrives.Simd.ConstraintHardIM), GlobalVectorConstants::FloatOne);
	
}

void FPBDJointCachedSolver::InitSwingTwistDrives(
	const FReal Dt,
	const FPBDJointSolverSettings& SolverSettings,
	const FPBDJointSettings& JointSettings,
	const bool bTwistDriveEnabled,
	const bool bSwing1DriveEnabled,
	const bool bSwing2DriveEnabled)
{
	FRotation3 R1Target = ConnectorRs[0] * JointSettings.AngularDrivePositionTarget;
	R1Target.EnforceShortestArcWith(ConnectorRs[1]);
	FRotation3 R1Error = R1Target.Inverse() * ConnectorRs[1];
	FVec3 R1TwistAxisError = R1Error * FJointConstants::TwistAxis();

	// Angle approximation Angle ~= Sin(Angle) for small angles, underestimates for large angles
	const FReal DTwistAngle = 2.0f * R1Error.X;
	const FReal DSwing1Angle = R1TwistAxisError.Y;
	const FReal DSwing2Angle = -R1TwistAxisError.Z;

	const int32 TW = (int32)EJointAngularConstraintIndex::Twist;
	const int32 S1 = (int32)EJointAngularConstraintIndex::Swing1;
	const int32 S2 = (int32)EJointAngularConstraintIndex::Swing2;

	// TODO this could be removed if bUseSimd
	RotationDrives.InitDatas(TW, true, FPBDJointUtilities::GetAngularTwistDriveStiffness(SolverSettings, JointSettings),
		FPBDJointUtilities::GetAngularTwistDriveDamping(SolverSettings, JointSettings), 0.0);
	RotationDrives.InitDatas(S1, true, FPBDJointUtilities::GetAngularSwingDriveStiffness(SolverSettings, JointSettings),
		FPBDJointUtilities::GetAngularSwingDriveDamping(SolverSettings, JointSettings), 0.0);
	RotationDrives.InitDatas(S2, true, FPBDJointUtilities::GetAngularSwingDriveStiffness(SolverSettings, JointSettings),
		FPBDJointUtilities::GetAngularSwingDriveDamping(SolverSettings, JointSettings), 0.0);

	RotationDrives.bAccelerationMode = FPBDJointUtilities::GetAngularDriveAccelerationMode(SolverSettings, JointSettings);

	const bool bUseTwistDrive = bTwistDriveEnabled && (((FMath::Abs(DTwistAngle) > AngleTolerance) && (RotationDrives.Data.ConstraintSoftStiffness[TW] > 0.0f)) || (RotationDrives.Data.ConstraintSoftDamping[TW] > 0.0f));
	const bool bUseSwing1Drive = bSwing1DriveEnabled && (((FMath::Abs(DSwing1Angle) > AngleTolerance) && (RotationDrives.Data.ConstraintSoftStiffness[S1] > 0.0f)) || (RotationDrives.Data.ConstraintSoftDamping[S1] > 0.0f));
	const bool bUseSwing2Drive = bSwing2DriveEnabled && (((FMath::Abs(DSwing2Angle) > AngleTolerance) && (RotationDrives.Data.ConstraintSoftStiffness[S2] > 0.0f)) || (RotationDrives.Data.ConstraintSoftDamping[S2] > 0.0f));
	RotationDrives.bUseSimd = bUseSimd && bUseTwistDrive && bUseSwing1Drive && bUseSwing2Drive;

	if (RotationDrives.bUseSimd)
	{
		FVec3 ConstraintAxes[3] = { ConnectorRs[1] * FJointConstants::TwistAxis(), ConnectorRs[1] * FJointConstants::Swing2Axis(), ConnectorRs[1] * FJointConstants::Swing1Axis() };
		InitRotationConstraintDriveSimd(ConstraintAxes, FRealSingle(Dt), FVec3(DTwistAngle, DSwing2Angle, DSwing1Angle));

		RotationDrives.Simd.ConstraintHardStiffness = VectorZeroFloat();
		RotationDrives.Simd.ConstraintLambda = VectorZeroFloat();

		RotationDrives.ConstraintVX[TW] = JointSettings.AngularDriveVelocityTarget[TW];
		RotationDrives.SetMaxForce(TW, JointSettings.AngularDriveMaxTorque[TW], Dt);
		RotationDrives.ConstraintVX[S1] = JointSettings.AngularDriveVelocityTarget[S1];
		RotationDrives.SetMaxForce(S1, JointSettings.AngularDriveMaxTorque[S1], Dt);
		RotationDrives.ConstraintVX[S2] = JointSettings.AngularDriveVelocityTarget[S2];
		RotationDrives.SetMaxForce(S2, JointSettings.AngularDriveMaxTorque[S2], Dt);
	}
	else
	{
		if (bUseTwistDrive)
		{
			InitRotationConstraintDrive(TW, ConnectorRs[1] * FJointConstants::TwistAxis(), Dt, DTwistAngle);
			RotationDrives.ConstraintVX[TW] = JointSettings.AngularDriveVelocityTarget[TW];
			RotationDrives.SetMaxForce(TW, JointSettings.AngularDriveMaxTorque[TW], Dt);
		}
		if (bUseSwing1Drive)
		{
			InitRotationConstraintDrive(S1, ConnectorRs[1] * FJointConstants::Swing1Axis(), Dt, DSwing1Angle);
			RotationDrives.ConstraintVX[S1] = JointSettings.AngularDriveVelocityTarget[S1];
			RotationDrives.SetMaxForce(S1, JointSettings.AngularDriveMaxTorque[S1], Dt);
		}
		if (bUseSwing2Drive)
		{
			InitRotationConstraintDrive(S2, ConnectorRs[1] * FJointConstants::Swing2Axis(), Dt, DSwing2Angle);
			RotationDrives.ConstraintVX[S2] = JointSettings.AngularDriveVelocityTarget[S2];
			RotationDrives.SetMaxForce(S2, JointSettings.AngularDriveMaxTorque[S2], Dt);
		}
	}
}

void FPBDJointCachedSolver::InitSLerpDrive(
	const FReal Dt,
	const FPBDJointSolverSettings& SolverSettings,
	const FPBDJointSettings& JointSettings)
{
	for (int32 AxisIndex = 0; AxisIndex < 3; ++AxisIndex)
	{
		RotationDrives.InitDatas(AxisIndex, true, FPBDJointUtilities::GetAngularSLerpDriveStiffness(SolverSettings, JointSettings),
			FPBDJointUtilities::GetAngularSLerpDriveDamping(SolverSettings, JointSettings), 0.0);
	}
	RotationDrives.bAccelerationMode = FPBDJointUtilities::GetAngularDriveAccelerationMode(SolverSettings, JointSettings);

	const FRotation3 R01 = ConnectorRs[0].Inverse() * ConnectorRs[1];
	FRotation3 TargetAngPos = JointSettings.AngularDrivePositionTarget;
	TargetAngPos.EnforceShortestArcWith(R01);
	const FRotation3 R1Error = TargetAngPos.Inverse() * R01;

	FVec3 AxisAngles =
	{
		2.0f * Utilities::AsinEst(R1Error.X),
		2.0f * Utilities::AsinEst(R1Error.Y),
		2.0f * Utilities::AsinEst(R1Error.Z)
	};

	FVec3 Axes[3];
	ConnectorRs[1].ToMatrixAxes(Axes[0], Axes[1], Axes[2]);

	if (RotationDrives.bUseSimd)
	{
		RotationDrives.Simd.ConstraintLambda = VectorZeroFloat();
		RotationDrives.Simd.ConstraintHardStiffness = VectorZeroFloat();
		InitRotationConstraintDriveSimd(Axes, FRealSingle(Dt), AxisAngles);
	}
	else
	{
		InitRotationConstraintDrive(0, Axes[0], Dt, AxisAngles[0]);
		InitRotationConstraintDrive(1, Axes[1], Dt, AxisAngles[1]);
		InitRotationConstraintDrive(2, Axes[2], Dt, AxisAngles[2]);
	}

	RotationDrives.SetMaxForce(0, JointSettings.AngularDriveMaxTorque[0], Dt);
	RotationDrives.SetMaxForce(1, JointSettings.AngularDriveMaxTorque[1], Dt);
	RotationDrives.SetMaxForce(2, JointSettings.AngularDriveMaxTorque[2], Dt);

	// @todo(chaos): pass constraint target velocity into InitRotationConstraintDrive (it currently sets it ConstraintVX to 0)
	if (!JointSettings.AngularDriveVelocityTarget.IsNearlyZero())
	{
		const FVec3 TargetAngVel = ConnectorRs[0] * JointSettings.AngularDriveVelocityTarget;
		RotationDrives.ConstraintVX[0] = FVec3::DotProduct(TargetAngVel, Axes[0]);
		RotationDrives.ConstraintVX[1] = FVec3::DotProduct(TargetAngVel, Axes[1]);
		RotationDrives.ConstraintVX[2] = FVec3::DotProduct(TargetAngVel, Axes[2]);
	}
}

/** APPLY ROTATION DRIVES *********************************************************************************/

void FPBDJointCachedSolver::ApplyRotationDrives(
	const FReal Dt)
{
	if (RotationDrives.bUseSimd)
	{
		ApplyRotationDrivesSimd(Dt);
	}
	else
	{
		for (int32 ConstraintIndex = 0; ConstraintIndex < 3; ++ConstraintIndex)
		{
			if (RotationDrives.GetValidDatas(ConstraintIndex))
			{
				ApplyAxisRotationDrive(ConstraintIndex, Dt);
			}
		}
	}
}

void FPBDJointCachedSolver::ApplyAxisRotationDrive(
	const int32 ConstraintIndex,
	const FReal Dt)
{
	const FReal Stiffness = RotationDrives.Data.ConstraintSoftStiffness[ConstraintIndex];
	const FReal Damping = RotationDrives.Data.ConstraintSoftDamping[ConstraintIndex];
	const FReal IM = RotationDrives.Data.ConstraintSoftIM[ConstraintIndex];

	// Stiffness position delta
	FReal CX = 0;
	if (Stiffness > UE_KINDA_SMALL_NUMBER)
	{
		const FReal DX = FVec3::DotProduct(Body(1).DQ() - Body(0).DQ(), RotationDrives.Data.ConstraintAxis[ConstraintIndex]);
		const FReal TargetX = RotationDrives.Data.ConstraintCX[ConstraintIndex];
		CX = TargetX + DX;
	}

	// Damping angular velocity delta
	FReal CVDt = 0;
	if (Damping > UE_KINDA_SMALL_NUMBER)
	{
		const FVec3 W0Dt = FVec3(Body(0).DQ()) + ConnectorWDts[0];
		const FVec3 W1Dt = FVec3(Body(1).DQ()) + ConnectorWDts[1];
		const FReal TargetW = RotationDrives.ConstraintVX[ConstraintIndex];
		CVDt = (TargetW * Dt) + FVec3::DotProduct(RotationDrives.Data.ConstraintAxis[ConstraintIndex], W0Dt - W1Dt);
	}

	FReal Lambda = RotationDrives.Data.ConstraintLambda[ConstraintIndex];
	FReal DeltaLambda = SolverStiffness * (Stiffness * CX - Damping * CVDt - Lambda) / IM;
	Lambda += DeltaLambda;

	RotationDrives.ApplyMaxLambda(ConstraintIndex, DeltaLambda, Lambda);
	RotationDrives.Data.ConstraintLambda[ConstraintIndex] = Lambda;

	SolveRotationConstraintDelta(ConstraintIndex, DeltaLambda, true, RotationDrives);
}

void FPBDJointCachedSolver::ApplyRotationDrivesSimd(
	const FReal Dtd)
{
	const VectorRegister4Float Body0DQ = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(Body(0).DQ()[0], Body(0).DQ()[1], Body(0).DQ()[2], 0.0f));
	const VectorRegister4Float Body1DQ = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(Body(1).DQ()[0], Body(1).DQ()[1], Body(1).DQ()[2], 0.0f));

	const VectorRegister4Float TargetW = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(RotationDrives.ConstraintVX[0], RotationDrives.ConstraintVX[1], RotationDrives.ConstraintVX[2], 0.0f));

	const VectorRegister4Float DQDiff = VectorSubtract(Body1DQ, Body0DQ);
	VectorRegister4Float DXArray[3];
	VectorRegister4Float CVDtArray[3];
	const VectorRegister4Float W0Dt = VectorAdd(Body0DQ, ConnectorWDtsSimd[0]);
	const VectorRegister4Float W1Dt = VectorAdd(Body1DQ, ConnectorWDtsSimd[1]);
	const VectorRegister4Float WDiff = VectorSubtract(W0Dt, W1Dt);

	for (int32 ConstraintIndex = 0; ConstraintIndex < 3; ConstraintIndex++)
	{
		DXArray[ConstraintIndex] = Private::VectorDot3FastX(DQDiff, RotationDrives.Simd.ConstraintAxis[ConstraintIndex]);
		CVDtArray[ConstraintIndex] = Private::VectorDot3FastX(RotationDrives.Simd.ConstraintAxis[ConstraintIndex], WDiff);
	}
	VectorRegister4Float DX = VectorUnpackLo(DXArray[0], DXArray[1]);
	DX = VectorMoveLh(DX, DXArray[2]);
	const VectorRegister4Float CX = VectorAdd(RotationDrives.Simd.ConstraintCX, DX);

	VectorRegister4Float CVDt = VectorUnpackLo(CVDtArray[0], CVDtArray[1]);
	CVDt = VectorMoveLh(CVDt, CVDtArray[2]);

	FRealSingle Dtf = FRealSingle(Dtd);
	VectorRegister4Float Dt = VectorLoadFloat1(&Dtf);

	CVDt = VectorMultiplyAdd(TargetW, Dt, CVDt);

	const FRealSingle SolverStiffnessf = FRealSingle(SolverStiffness);
	VectorRegister4Float Stiffness = VectorLoadFloat1(&SolverStiffnessf);
	VectorRegister4Float DeltaLambda = VectorMultiply(Stiffness, VectorDivide(VectorSubtract(VectorMultiply(RotationDrives.Simd.ConstraintSoftStiffness, CX),
		VectorMultiplyAdd(RotationDrives.Simd.ConstraintSoftDamping, CVDt, RotationDrives.Simd.ConstraintLambda)), RotationDrives.Simd.ConstraintSoftIM));
	RotationDrives.Simd.ConstraintLambda = VectorAdd(RotationDrives.Simd.ConstraintLambda, DeltaLambda);

	// Check and apply MaxLambda 
	RotationDrives.ApplyMaxLambdaSIMD(DeltaLambda, RotationDrives.Simd.ConstraintLambda);

	VectorRegister4Float DeltaLambdas[3];
	DeltaLambdas[0] = VectorReplicate(DeltaLambda, 0);
	DeltaLambdas[1] = VectorReplicate(DeltaLambda, 1);
	DeltaLambdas[2] = VectorReplicate(DeltaLambda, 2);

	VectorRegister4Float DeltaImpulses[3];
	for (int32 ConstraintIndex = 0; ConstraintIndex < 3; ConstraintIndex++)
	{
		DeltaImpulses[ConstraintIndex] = VectorMultiply(RotationDrives.Simd.ConstraintAxis[ConstraintIndex], DeltaLambdas[ConstraintIndex]);
	}

	if (Body(0).IsDynamic())
	{
		VectorRegister4Float DR0 = VectorZeroFloat();
		for (int32 ConstraintIndex = 0; ConstraintIndex < 3; ConstraintIndex++)
		{
			const VectorRegister4Float Axis = VectorDot3(RotationDrives.Simd.ConstraintAxis[ConstraintIndex], RotationDrives.Simd.ConstraintDRAxis[ConstraintIndex][0]);
			DR0 = VectorMultiplyAdd(DeltaImpulses[ConstraintIndex], Axis, DR0);
		}
		FVec3f DR0f;
		VectorStoreFloat3(DR0, &DR0f[0]);
		ApplyRotationDelta(0, FVec3(DR0f));
	}
	if (Body(1).IsDynamic())
	{
		VectorRegister4Float DR1 = VectorZeroFloat();
		for (int32 ConstraintIndex = 0; ConstraintIndex < 3; ConstraintIndex++)
		{
			const VectorRegister4Float Axis = VectorDot3(RotationDrives.Simd.ConstraintAxis[ConstraintIndex], RotationDrives.Simd.ConstraintDRAxis[ConstraintIndex][1]);
			DR1 = VectorMultiplyAdd(DeltaImpulses[ConstraintIndex], Axis, DR1);
		}
		FVec3f DR1f;
		VectorStoreFloat3(DR1, &DR1f[0]);
		ApplyRotationDelta(1, FVec3(DR1f));
	}
	NumActiveConstraints += 3;
}

void FPBDJointCachedSolver::ApplyRotationVelocityDrives(
	const FReal Dt)
{
	if (bUsePositionBasedDrives)
	{
		return;
	}

	for (int32 ConstraintIndex = 0; ConstraintIndex < 3; ++ConstraintIndex)
	{
		if (RotationDrives.GetValidDatas(ConstraintIndex))
		{
			ApplyAxisRotationVelocityDrive(ConstraintIndex, Dt);
		}
	}
}

void FPBDJointCachedSolver::ApplyAxisRotationVelocityDrive(
	const int32 ConstraintIndex,
	const FReal Dt)
{
	check(RotationDrives.bUseSimd == false);
	// NOTE: Using the actual damping, not the PBD modified value
	const FReal Damping = RotationDrives.SettingsSoftDamping[ConstraintIndex] * Dt;
	if (Damping < UE_SMALL_NUMBER)
	{
		return;
	}

	const FReal MassScale = RotationDrives.bAccelerationMode ? (FReal(1) / RotationDrives.Data.ConstraintHardIM[ConstraintIndex]) : FReal(1);
	const FReal IM = MassScale * Damping * RotationDrives.Data.ConstraintHardIM[ConstraintIndex] + (FReal)1;

	// Angular velocity error to correct
	const FReal WRel = FVec3::DotProduct(W(1) - W(0), RotationDrives.Data.ConstraintAxis[ConstraintIndex]);
	const FReal TargetW = RotationDrives.ConstraintVX[ConstraintIndex];
	const FReal CV = (WRel - TargetW);

	// Implicit scheme: F(t) = -D x W(t+dt)
	FReal& LambdaVel = RotationDrives.ConstraintLambdaVelocity[ConstraintIndex];
	FReal DeltaLambdaVel = SolverStiffness * (MassScale * Damping * CV - LambdaVel) / IM;

	// Apply limits and accumulate total impulse 
	// (NOTE: Limits and net impulses are position based, not velocity based)
	FReal DeltaLambda = DeltaLambdaVel * Dt;
	FReal Lambda = RotationDrives.Data.ConstraintLambda[ConstraintIndex] + DeltaLambda;
	RotationDrives.ApplyMaxLambda(ConstraintIndex, DeltaLambda, Lambda);
	RotationDrives.Data.ConstraintLambda[ConstraintIndex] = Lambda;
	DeltaLambdaVel = DeltaLambda / Dt;

	LambdaVel += DeltaLambdaVel;
	const FVec3 Impulse = DeltaLambdaVel * RotationDrives.Data.ConstraintAxis[ConstraintIndex];

	if (Body(0).IsDynamic())
	{
		const FVec3 DW0 = RotationDrives.Data.ConstraintDRAxis[ConstraintIndex][0] * DeltaLambdaVel;
		Body(0).ApplyAngularVelocityDelta(DW0);
	}
	if (Body(1).IsDynamic())
	{
		const FVec3 DW1 = RotationDrives.Data.ConstraintDRAxis[ConstraintIndex][1] * DeltaLambdaVel;
		Body(1).ApplyAngularVelocityDelta(DW1);
	}
}

// Joint utilities

void FPBDJointCachedSolver::ApplyPositionDelta(
	const int32 BodyIndex,
	const FVec3& DP)
{
	Body(BodyIndex).ApplyPositionDelta(DP);
}

void FPBDJointCachedSolver::ApplyRotationDelta(
	const int32 BodyIndex,
	const FVec3& DR)
{
	Body(BodyIndex).ApplyRotationDelta(DR);
}

void FAxisConstraintDatas::InitDatas(
	const int32 ConstraintIndex,
	const bool bHasSoftLimits,
	const FReal SoftStiffness,
	const FReal SoftDamping,
	const FReal HardStiffness,
	const bool bResetLambdas)
{
	SetSoftLimit(ConstraintIndex, bHasSoftLimits);
	Data.ConstraintHardStiffness[ConstraintIndex] = HardStiffness;
	Data.ConstraintSoftStiffness[ConstraintIndex] = SoftStiffness;
	Data.ConstraintSoftDamping[ConstraintIndex] = SoftDamping;
	ConstraintMaxLambda[ConstraintIndex] = 0;
	SettingsSoftStiffness[ConstraintIndex] = SoftStiffness;
	SettingsSoftDamping[ConstraintIndex] = SoftDamping;
	SetValidDatas(ConstraintIndex, false);
	SetLimitsCheck(ConstraintIndex, true);
	SetMotionType(ConstraintIndex, EJointMotionType::Free);
	if (bResetLambdas)
	{
		Data.ConstraintLambda = FVec3::Zero();
		ConstraintLambdaVelocity = FVec3::Zero();
		Data.ConstraintLimits = FVec3::Zero();
	}
}

void FAxisConstraintDatas::UpdateDatas(
	const int32 ConstraintIndex,
	const FVec3& DatasAxis,
	const FReal DatasCX,
	const FReal DatasRestitution,
	const bool bCheckLimit,
	const FVec3& DatasArm0,
	const FVec3& DatasArm1,
	const FReal DatasVX)
{
	SetValidDatas(ConstraintIndex, true);
	SetLimitsCheck(ConstraintIndex, bCheckLimit);

	Data.ConstraintCX[ConstraintIndex] = DatasCX;
	ConstraintVX[ConstraintIndex] = DatasVX;
	ConstraintRestitution[ConstraintIndex] = DatasRestitution;
	Data.ConstraintArms[ConstraintIndex][0] = DatasArm0;
	Data.ConstraintArms[ConstraintIndex][1] = DatasArm1;
	Data.ConstraintAxis[ConstraintIndex] = DatasAxis;
}

void FAxisConstraintDatas::UpdateMass(
	const int32 ConstraintIndex,
	const FVec3& DatasIA0,
	const FVec3& DatasIA1,
	const FReal DatasIM,
	const FReal Dt,
	const bool bUsePositionBasedDrives)
{
	Data.ConstraintHardIM[ConstraintIndex] = DatasIM;


	Data.ConstraintDRAxis[ConstraintIndex][0] = DatasIA0;
	Data.ConstraintDRAxis[ConstraintIndex][1] = -DatasIA1;

	if (GetSoftLimit(ConstraintIndex))
	{
		// If bUsePositionBasedDrives is false, we apply the velocity drive in the velocity solver phase so we don't include in the PBD settings
		const FReal SpringMassScale = (bAccelerationMode) ? (FReal)1 / (Data.ConstraintHardIM[ConstraintIndex]) : (FReal)1;
		Data.ConstraintSoftStiffness[ConstraintIndex] = SpringMassScale * SettingsSoftStiffness[ConstraintIndex] * Dt * Dt;
		Data.ConstraintSoftDamping[ConstraintIndex] = (bUsePositionBasedDrives) ? SpringMassScale * SettingsSoftDamping[ConstraintIndex] * Dt : 0;
		Data.ConstraintSoftIM[ConstraintIndex] = (Data.ConstraintSoftStiffness[ConstraintIndex] + Data.ConstraintSoftDamping[ConstraintIndex]) * Data.ConstraintHardIM[ConstraintIndex] + (FReal)1;
	}
}

void FAxisConstraintDatas::SetMaxForce(
	const int32 ConstraintIndex,
	const FReal InMaxForce,
	const FReal Dt)
{
	// We use 0 to disable max force clamping. See ApplyMaxLambda
	ConstraintMaxLambda[ConstraintIndex] = 0;

	if ((InMaxForce > 0) && (InMaxForce < UE_MAX_FLT))
	{
		// Convert from force/torque to position/angle impulse
		FReal MaxLambda = InMaxForce * Dt * Dt;
		if (bAccelerationMode)
		{
			FReal ConstraintIM = Data.ConstraintHardIM[ConstraintIndex];
			if (bUseSimd)
			{
				ConstraintIM = FReal(VectorGetComponentDynamic(Simd.ConstraintHardIM, ConstraintIndex));
			}
			if (ConstraintIM > UE_SMALL_NUMBER)
			{
				MaxLambda /= ConstraintIM;
			}
		}
		ConstraintMaxLambda[ConstraintIndex] = MaxLambda;
	}
}

void FAxisConstraintDatas::ApplyMaxLambda(
	const int32 ConstraintIndex,
	FReal& DeltaLambda,
	FReal& Lambda)
{
	if (ConstraintMaxLambda[ConstraintIndex] > 0)
	{
		if (Lambda > ConstraintMaxLambda[ConstraintIndex])
		{
			DeltaLambda = ConstraintMaxLambda[ConstraintIndex] - Data.ConstraintLambda[ConstraintIndex];
			Lambda = ConstraintMaxLambda[ConstraintIndex];
		}
		else if (Lambda < -ConstraintMaxLambda[ConstraintIndex])
		{
			DeltaLambda = -ConstraintMaxLambda[ConstraintIndex] - Data.ConstraintLambda[ConstraintIndex];
			Lambda = -ConstraintMaxLambda[ConstraintIndex];
		}
	}
}

void FAxisConstraintDatas::ApplyMaxLambdaSIMD(
	VectorRegister4Float& DeltaLambda,
	VectorRegister4Float& Lambda)
{
	// 1. Load MaxLambdas
	VectorRegister4Float MaxLambda = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(ConstraintMaxLambda[0], ConstraintMaxLambda[1], ConstraintMaxLambda[2], 0.0f));

	// 2. Prepare -MaxLambda
	VectorRegister4Float NegMaxLambda = VectorNegate(MaxLambda);

	// 3. Create a mask where MaxLambda is greater than zero
	VectorRegister4Float Zero = VectorZero();
	VectorRegister4Float IsActiveMask = VectorCompareGT(MaxLambda, Zero);

	// 4. If (Lambda > MaxLambda) -> Clamp Top
	VectorRegister4Float TooHighMask = VectorCompareGT(Lambda, MaxLambda);

	// 5. else if (Lambda < -MaxLambda) -> Clamp Bottom
	VectorRegister4Float TooLowMask = VectorCompareLT(Lambda, NegMaxLambda);

	// 6. Calculate clamped Lambda values
	// FinalLambda = (Lambda > Max) ? Max : ((Lambda < -Max) ? -Max : Lambda)
	VectorRegister4Float ClampedLambda = VectorSelect(TooHighMask, MaxLambda, Lambda);
	ClampedLambda = VectorSelect(TooLowMask, NegMaxLambda, ClampedLambda);

	// 7. Apply check (ConstraintMaxLambda > 0)
	VectorRegister4Float OriginalLambda = Lambda;
	Lambda = VectorSelect(IsActiveMask, ClampedLambda, Lambda);

	// 8. Adjust DeltaLambda
	DeltaLambda = VectorAdd(DeltaLambda, VectorSubtract(Lambda, OriginalLambda));
}

}


