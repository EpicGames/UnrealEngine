// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/Joint/PBDJointContainerSolver.h"
#include "Chaos/Joint/ChaosJointLog.h"
#include "Chaos/Island/IslandManager.h"
#include "Chaos/DebugDrawQueue.h"

namespace Chaos
{
	namespace Private
	{
		///////////////////////////////////////////////////////////////////////////////////////////
		///////////////////////////////////////////////////////////////////////////////////////////
		///////////////////////////////////////////////////////////////////////////////////////////

		// NOTE: Particles are passed to the solvers in reverse order to what they are in the container...
		FGeometryParticleHandle* GetJointParticle(FPBDJointConstraints& Constraints, const int32 ContainerConstraintIndex, const int32 ParticleIndex)
		{
			check(ParticleIndex >= 0);
			check(ParticleIndex < 2);

			const int32 SwappedIndex = 1 - ParticleIndex;
			return Constraints.GetConstrainedParticles(ContainerConstraintIndex)[SwappedIndex];
		}

		const FRigidTransform3& GetJointFrame(FPBDJointConstraints& Constraints, const int32 ContainerConstraintIndex, const int32 ParticleIndex)
		{
			check(ParticleIndex >= 0);
			check(ParticleIndex < 2);

			const int32 SwappedIndex = 1 - ParticleIndex;
			return Constraints.GetConstraintSettings(ContainerConstraintIndex).ConnectorTransforms[SwappedIndex];
		}


		// @todo(chaos): ShockPropagation needs to handle the parent/child being in opposite order
		FReal GetJointShockPropagationInvMassScale(const FConstraintSolverBody& Body0, const FConstraintSolverBody& Body1, const FPBDJointSolverSettings& Settings, const FPBDJointSettings& JointSettings, const int32 It, const int32 NumIts)
		{
			// Shock propagation is only enabled for the last iteration, and only for the QPBD solver.
			// The standard PBD solver runs projection in the second solver phase which is mostly the same thing.
			if (JointSettings.bShockPropagationEnabled && (It >= (NumIts - Settings.NumShockPropagationIterations)))
			{
				if (Body0.IsDynamic() && Body1.IsDynamic())
				{
					return FPBDJointUtilities::GetShockPropagationInvMassScale(Settings, JointSettings);
				}
			}
			return FReal(1);
		}

		FReal GetJointIterationStiffness(const FPBDJointSolverSettings& Settings, int32 It, int32 NumIts)
		{
			// Linearly interpolate betwwen MinStiffness and MaxStiffness over the first few iterations,
			// then clamp at MaxStiffness for the final NumIterationsAtMaxStiffness
			FReal IterationStiffness = Settings.MaxSolverStiffness;
			if (NumIts > Settings.NumIterationsAtMaxSolverStiffness)
			{
				const FReal Interpolant = FMath::Clamp((FReal)It / (FReal)(NumIts - Settings.NumIterationsAtMaxSolverStiffness), 0.0f, 1.0f);
				IterationStiffness = FMath::Lerp(Settings.MinSolverStiffness, Settings.MaxSolverStiffness, Interpolant);
			}
			return FMath::Clamp(IterationStiffness, 0.0f, 1.0f);
		}

		bool GetJointShouldBreak(const FPBDJointSettings& JointSettings, const FReal Dt, const FVec3& LinearImpulse, const FVec3& AngularImpulse)
		{
			// NOTE: LinearImpulse/AngularImpulse are not really impulses - they are mass-weighted position/rotation delta, or (impulse x dt).
			// The Threshold is a force limit, so we need to convert it to a position delta caused by that force in one timestep

			bool bBreak = false;
			if (!bBreak && JointSettings.LinearBreakForce != FLT_MAX)
			{
				const FReal LinearForceSq = LinearImpulse.SizeSquared() / (Dt * Dt * Dt * Dt);
				const FReal LinearThresholdSq = FMath::Square(JointSettings.LinearBreakForce);
				bBreak = LinearForceSq > LinearThresholdSq;
			}

			if (!bBreak && JointSettings.AngularBreakTorque != FLT_MAX)
			{
				const FReal AngularForceSq = AngularImpulse.SizeSquared() / (Dt * Dt * Dt * Dt);
				const FReal AngularThresholdSq = FMath::Square(JointSettings.AngularBreakTorque);
				bBreak = AngularForceSq > AngularThresholdSq;
			}

			return bBreak;
		}

		bool GetJointIsViolating(const FPBDJointSettings& JointSettings, const FReal LinearViolationSq, const FReal AngularViolation, const int32 It, const int32 NumIts)
		{
			bool bViolating = false;
			if (It == NumIts - 1)
			{
				const FReal LinearViolationCallbackThresholdSq
					= JointSettings.LinearViolationCallbackThreshold
					* JointSettings.LinearViolationCallbackThreshold;
				if (!bViolating && JointSettings.LinearViolationCallbackThreshold != FLT_MAX)
				{
					bViolating = LinearViolationSq > LinearViolationCallbackThresholdSq;
				}

				if (!bViolating && JointSettings.AngularViolationCallbackThreshold != FLT_MAX)
				{
					bViolating = FMath::RadiansToDegrees(AngularViolation) > JointSettings.AngularViolationCallbackThreshold;
				}
			}

			return bViolating;
		}

		///////////////////////////////////////////////////////////////////////////////////////////
		///////////////////////////////////////////////////////////////////////////////////////////
		///////////////////////////////////////////////////////////////////////////////////////////

		template <typename JointSolverType>
		TPBDJointContainerSolver<JointSolverType>::TPBDJointContainerSolver(FPBDJointConstraints& InConstraintContainer, const int32 InPriority)
			: FConstraintContainerSolver(InPriority)
			, ConstraintContainer(InConstraintContainer)
		{
		}

		template <typename JointSolverType>
		TPBDJointContainerSolver<JointSolverType>::~TPBDJointContainerSolver()
		{
		}

		template <typename JointSolverType>
		void TPBDJointContainerSolver<JointSolverType>::ResetImpl(const int32 InMaxConstraints)
		{
			ConstraintSolvers.SetNum(InMaxConstraints);
			ContainerIndices.Reset(InMaxConstraints);
		}

		template <typename JointSolverType>
		void TPBDJointContainerSolver<JointSolverType>::AddConstraints()
		{
			Reset(ConstraintContainer.GetNumConstraints());

			// @todo(chaos): we could eliminate the index array if we're solving all constraints in the scene (RBAN)
			for (int32 ContainerConstraintIndex = 0; ContainerConstraintIndex < ConstraintContainer.GetNumConstraints(); ++ContainerConstraintIndex)
			{
				if (ConstraintContainer.IsConstraintEnabled(ContainerConstraintIndex))
				{
					AddConstraint(ContainerConstraintIndex);
				}
			}
		}

		template <typename JointSolverType>
		void TPBDJointContainerSolver<JointSolverType>::AddConstraints(const TArrayView<Private::FPBDIslandConstraint*>& IslandConstraints)
		{
			int32 CurrentIndex = 0;
			for (Private::FPBDIslandConstraint* IslandConstraint : IslandConstraints)
			{
				// Filter out sleeping constraints in any partially sleeping island
				if (!IslandConstraint->IsSleeping())
				{
					FConstraintHandle* Constraint = IslandConstraint->GetConstraint();

					// We will only ever be given constraints from our container (asserts in non-shipping)
					const int32 ContainerConstraintIndex = Constraint->AsUnsafe<FPBDJointConstraintHandle>()->GetConstraintIndex();

					AddConstraint(ContainerConstraintIndex);
				}
			}
		}

		template <typename JointSolverType>
		void TPBDJointContainerSolver<JointSolverType>::AddConstraint(const int32 InContainerConstraintIndex)
		{
			// If this triggers, Reset was called with the wrong constraint count
			check(ContainerIndices.Num() <= ConstraintSolvers.Num());

			// Only add a constraint if it is working on at least one dynamic body
			const FGenericParticleHandle Particle0 = GetJointParticle(ConstraintContainer, InContainerConstraintIndex, 0);
			const FGenericParticleHandle Particle1 = GetJointParticle(ConstraintContainer, InContainerConstraintIndex, 1);

			if (Particle0->IsDynamic() || Particle1->IsDynamic())
			{
				ContainerIndices.Add(InContainerConstraintIndex);
			}
		}

		template <typename JointSolverType>
		bool TPBDJointContainerSolver<JointSolverType>::SortConstraintsByPartitionIndex(const TArray<int32>& PartitionIndices, const TArray<int32>& PrefixSum)
		{
			Private::SortArrayByPartitionIndex(ContainerIndices, PartitionIndices, PrefixSum);
			return true;
		}

		template <typename JointSolverType>
		void TPBDJointContainerSolver<JointSolverType>::GetAllConstrainedParticles(TArray<const FGeometryParticleHandle*>& ParticleHandles) const
		{
			ParticleHandles.Reset();
			ParticleHandles.Reserve(ContainerIndices.Num() * 2);

			for (int32 Index : ContainerIndices)
			{
				const FParticlePair& ParticlePair = ConstraintContainer.GetConstrainedParticles(Index);
				ParticleHandles.Add(ParticlePair[0]);
				ParticleHandles.Add(ParticlePair[1]);
			}
		}

		template <typename SolverType>
		void AddBodiesImpl(const TPBDJointContainerSolver<SolverType>& Container, const TArray<int32>& SolverGlobalIndices, FSolverBodyContainer& SolverBodyContainer, TArray<SolverType>& Solvers)
		{
			for (int32 i = 0; i < SolverGlobalIndices.Num(); i++)
			{
				const int32 ContainerConstraintIndex = SolverGlobalIndices[i];
				FGenericParticleHandle Particle0 = GetJointParticle(Container.GetContainer(), ContainerConstraintIndex, 0);
				FGenericParticleHandle Particle1 = GetJointParticle(Container.GetContainer(), ContainerConstraintIndex, 1);

				FSolverBody* SolverBody0 = SolverBodyContainer.FindOrAdd(Particle0);
				FSolverBody* SolverBody1 = SolverBodyContainer.FindOrAdd(Particle1);

				Solvers[i].SetSolverBodies(SolverBody0, SolverBody1);
			}

		}

		template <typename JointSolverType>
		void TPBDJointContainerSolver<JointSolverType>::AddBodies(FSolverBodyContainer& SolverBodyContainer)
		{
			AddBodiesImpl(*this, ContainerIndices, SolverBodyContainer, ConstraintSolvers);
			ResizeSolverArrays();
		}

		template <typename JointSolverType>
		void TPBDJointContainerSolver<JointSolverType>::GatherInput(const FReal Dt)
		{
			GatherInput(Dt, 0, GetNumConstraints());
		}

		template<typename SolverType>
		void GatherInputImpl(const TPBDJointContainerSolver<SolverType>& Container, TArray<SolverType>& Solvers, const TArray<int32>& SolverGlobalIndices, const FReal Dt, const int32 SolverConstraintBeginIndex, const int32 SolverConstraintEndIndex, const bool bUseLinearSolver)
		{
			const FPBDJointSolverSettings& SolverSettings = Container.GetSettings();

			for (int32 SolverConstraintIndex = FMath::Max(0, SolverConstraintBeginIndex); SolverConstraintIndex < FMath::Min(SolverConstraintEndIndex, SolverGlobalIndices.Num()); ++SolverConstraintIndex)
			{
				const FPBDJointSettings& JointSettings = Container.GetConstraintSettings(SolverConstraintIndex);

				const int32 ContainerConstraintIndex = Container.GetContainerConstraintIndex(SolverConstraintIndex);
				FGenericParticleHandle Particle0 = GetJointParticle(Container.GetContainer(), ContainerConstraintIndex, 0);
				FGenericParticleHandle Particle1 = GetJointParticle(Container.GetContainer(), ContainerConstraintIndex, 1);
				const FRigidTransform3& Frame0 = GetJointFrame(Container.GetContainer(), ContainerConstraintIndex, 0);
				const FRigidTransform3& Frame1 = GetJointFrame(Container.GetContainer(), ContainerConstraintIndex, 1);
				Solvers[SolverConstraintIndex].Init(Dt, SolverSettings, JointSettings, Particle0->GetComRelativeTransform(Frame0), Particle1->GetComRelativeTransform(Frame1));
			}
		}

		template<typename SolverType>
		void TPBDJointContainerSolver<SolverType>::GatherInput(const FReal Dt, const int32 ConstraintBeginIndex, const int32 ConstraintEndIndex)
		{
			GatherInputImpl(*this, ConstraintSolvers, ContainerIndices, Dt, ConstraintBeginIndex, ConstraintEndIndex, true);
		}

		template<typename SolverType>
		void TPBDJointContainerSolver<SolverType>::ScatterOutput(const FReal Dt)
		{
			ScatterOutput(Dt, 0, GetNumConstraints());
		}

		template<typename SolverType>
		void ScatterOutputImpl(const TPBDJointContainerSolver<SolverType>& Container, TArray<SolverType>& Solvers, const TArray<int32>& SolverGlobalIndices, const FReal Dt, const int32 SolverConstraintBeginIndex, const int32 SolverConstraintEndIndex, const bool bUseLinearSolver)
		{
			for (int32 SolverConstraintIndex = FMath::Max(0, SolverConstraintBeginIndex); SolverConstraintIndex < FMath::Min(SolverConstraintEndIndex, SolverGlobalIndices.Num()); ++SolverConstraintIndex)
			{
				const int32 GlobalContainerConstraintIndex = Container.GetContainerConstraintIndex(SolverConstraintIndex);
				if (Dt > UE_SMALL_NUMBER)
				{
					SolverType& Solver = Solvers[SolverConstraintIndex];
					// NOTE: Particle order was revered in the solver...
					// NOTE: Solver impulses are positional impulses
					const FVec3 LinearImpulse = -Solver.GetNetLinearImpulse() / Dt;
					const FVec3 AngularImpulse = -Solver.GetNetAngularImpulse() / Dt;
					const FSolverBody* SolverBody0 = &Solver.Body0().SolverBody();
					const FSolverBody* SolverBody1 = &Solver.Body1().SolverBody();
					const bool bIsBroken = Solver.IsBroken();

					const bool bIsViolating = Solver.IsViolating();
					const float LinearViolation = bIsViolating ? FMath::Sqrt((float)Solver.GetLinearViolationSq()) : 0.f;
					const float AngularViolation = bIsViolating ? (float)Solver.GetAngularViolation() : 0.f;

					Container.GetContainer().SetSolverResults(GlobalContainerConstraintIndex, LinearImpulse, AngularImpulse, LinearViolation, AngularViolation, bIsBroken, bIsViolating, SolverBody0, SolverBody1);

					Solver.Deinit();
				}
				else
				{
					Container.GetContainer().SetSolverResults(GlobalContainerConstraintIndex, FVec3(0), FVec3(0), 0.f, 0.f, false, false, nullptr, nullptr);
				}
			}
		}

		template<typename SolverType>
		void TPBDJointContainerSolver<SolverType>::ScatterOutput(const FReal Dt, const int32 ConstraintBeginIndex, const int32 ConstraintEndIndex)
		{
			ScatterOutputImpl(*this, ConstraintSolvers, ContainerIndices, Dt, ConstraintBeginIndex, ConstraintEndIndex, true);
		}

		template<typename SolverType>
		void TPBDJointContainerSolver<SolverType>::ResizeSolverArrays()
		{
			check(ConstraintSolvers.Num() >= ContainerIndices.Num());
			ConstraintSolvers.SetNum(ContainerIndices.Num());
		}

		// Apply position constraints for linear or non-linear solvers
		template<typename SolverType>
		void ApplyPositionConstraintsImpl(const TPBDJointContainerSolver<SolverType>& Container, TArray<SolverType>& Solvers, const TArray<int32>& SolverGlobalIndices, 
			const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex)
		{
			const FPBDJointSolverSettings& Settings = Container.GetSettings();
			const FReal IterationStiffness = GetJointIterationStiffness(Settings, It, NumIts);

			for (int32 SolverConstraintIndex = FMath::Max(0, BeginIndex); SolverConstraintIndex < FMath::Min(EndIndex, Solvers.Num()); ++SolverConstraintIndex)
			{
				SolverType& Solver = Solvers[SolverConstraintIndex];
				if (!Solver.RequiresSolve())
				{
					continue;
				}

				const FPBDJointSettings& JointSettings = Container.GetConstraintSettings(SolverConstraintIndex);
				Solver.Update(Dt, Settings, JointSettings);

				// Set parent inverse mass scale based on current shock propagation state
				const FReal ShockPropagationInvMassScale = GetJointShockPropagationInvMassScale(Solver.Body0(), Solver.Body1(), Settings, JointSettings, It, NumIts);
				Solver.SetShockPropagationScales(ShockPropagationInvMassScale, FReal(1), Dt);

				Solver.ApplyConstraints(Dt, IterationStiffness, Settings, JointSettings);

				// @todo(ccaulfield): We should be clamping the impulse at this point. Maybe move breaking to the solver
				if (GetJointShouldBreak(JointSettings, Dt, Solver.GetNetLinearImpulse(), Solver.GetNetAngularImpulse()))
				{
					Solver.SetIsBroken(true);
				}

				Solver.SetIsViolating(GetJointIsViolating(JointSettings, Solver.GetLinearViolationSq(), Solver.GetAngularViolation(), It, NumIts));
			}
		}

		template<typename SolverType>
		void TPBDJointContainerSolver<SolverType>::ApplyPositionConstraints(const FReal Dt, const int32 It, const int32 NumIts)
		{
			// ResizeSolverArrays() called in AddBodies() after adding all constraints and solvers.
			check(ConstraintSolvers.Num() == ContainerIndices.Num());
			ApplyPositionConstraints(Dt, It, NumIts, 0, GetNumConstraints());
		}

		template<typename SolverType>
		void TPBDJointContainerSolver<SolverType>::ApplyPositionConstraints(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex)
		{
			// ResizeSolverArrays() called in AddBodies() after adding all constraints and solvers.
			check(ConstraintSolvers.Num() == ContainerIndices.Num());
 			ApplyPositionConstraintsImpl(*this, ConstraintSolvers, ContainerIndices, Dt, It, NumIts, BeginIndex, EndIndex);
		}

		// Apply velocity constraints for linear or non-linear solvers
		template<typename SolverType>
		void ApplyVelocityConstraintsImpl(const TPBDJointContainerSolver<SolverType>& Container, TArray<SolverType>& Solvers, const FReal Dt, const int32 It, const int32 NumIts,
			const int32 BeginIndex, const int32 EndIndex)
		{
			const FPBDJointSolverSettings& Settings = Container.GetSettings();
			const FReal IterationStiffness = GetJointIterationStiffness(Settings, It, NumIts);

			for (int32 SolverConstraintIndex = FMath::Max(0, BeginIndex); SolverConstraintIndex < FMath::Min(EndIndex, Solvers.Num()); ++SolverConstraintIndex)
			{
				SolverType& Solver = Solvers[SolverConstraintIndex];
				if (!Solver.RequiresSolve())
				{
					continue;
				}

				const FPBDJointSettings& JointSettings = Container.GetConstraintSettings(SolverConstraintIndex);
				Solver.Update(Dt, Settings, JointSettings);

				// Set parent inverse mass scale based on current shock propagation state
				const FReal ShockPropagationInvMassScale = GetJointShockPropagationInvMassScale(Solver.Body0(), Solver.Body1(), Settings, JointSettings, It, NumIts);
				Solver.SetShockPropagationScales(ShockPropagationInvMassScale, FReal(1), Dt);

				Solver.ApplyVelocityConstraints(Dt, IterationStiffness, Settings, JointSettings);

				// @todo(chaos): should also add to net impulse and run break logic
			}
		}

		template<typename SolverType>
		void TPBDJointContainerSolver<SolverType>::ApplyVelocityConstraints(const FReal Dt, const int32 It, const int32 NumIts)
		{
			ApplyVelocityConstraints(Dt, It, NumIts, 0, ConstraintSolvers.Num());
		}

		template<typename SolverType>
		void TPBDJointContainerSolver<SolverType>::ApplyVelocityConstraints(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex)
		{
			ApplyVelocityConstraintsImpl(*this, ConstraintSolvers, Dt, It, NumIts, BeginIndex, EndIndex);
		}

		template<typename SolverType>
		void TPBDJointContainerSolver<SolverType>::PreApplyProjectionConstraints(const FReal Dt)
		{
			PreApplyProjectionConstraints(Dt, 0, ConstraintSolvers.Num());
		}

		template<typename SolverType>
		void TPBDJointContainerSolver<SolverType>::PreApplyProjectionConstraints(const FReal Dt, const int32 BeginIndex, const int32 EndIndex)
		{
			if (UseLinearSolver())
			{
				PreApplyLinearProjectionConstraints(Dt, BeginIndex, EndIndex);
			}
			else
			{
				PreApplyNonLinearProjectionConstraints(Dt, BeginIndex, EndIndex);
			}
		}

		template<typename SolverType>
		void TPBDJointContainerSolver<SolverType>::ApplyProjectionConstraints(const FReal Dt, const int32 It, const int32 NumIts)
		{
			ApplyProjectionConstraints(Dt, It, NumIts, 0, ConstraintSolvers.Num());
		}

		template<typename SolverType>
		void TPBDJointContainerSolver<SolverType>::ApplyProjectionConstraints(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex)
		{
			if (UseLinearSolver())
			{
				ApplyLinearProjectionConstraints(Dt, It, NumIts, BeginIndex, EndIndex);
			}
			else
			{
				ApplyNonLinearProjectionConstraints(Dt, It, NumIts, BeginIndex, EndIndex);
			}
		}

		template <>
		bool TPBDJointContainerSolver<FPBDJointCachedSolver>::UseLinearSolver() const
		{
			return true;
		}

		template <>
		bool TPBDJointContainerSolver<FPBDJointSolver>::UseLinearSolver() const
		{
			return false;
		}

		template<>
		void TPBDJointContainerSolver<FPBDJointCachedSolver>::PreApplyLinearProjectionConstraints(const FReal Dt, const int32 BeginIndex, const int32 EndIndex)
		{
			constexpr bool bUseSolveLinear = true;
			const FPBDJointSolverSettings& Settings = GetSettings();

			// Collect all the data for projection prior to the first iteration. 
			// This must happen for all joints before we project any joints so the the initial state for each joint is not polluted by any earlier projections.
			// @todo(chaos): if we ever support projection on other constraint types, we will need a PrepareProjection phase so that all constraint types
			// can initialize correctly before any constraints apply their projection. For now we can just check the iteration count is zero.
			for (int32 SolverConstraintIndex = FMath::Max(0, BeginIndex); SolverConstraintIndex < FMath::Min(EndIndex, ConstraintSolvers.Num()); ++SolverConstraintIndex)
			{
				FPBDJointCachedSolver& Solver = ConstraintSolvers[SolverConstraintIndex];
				if (!Solver.RequiresSolve())
				{
					continue;
				}

				const FPBDJointSettings& JointSettings = GetConstraintSettings(SolverConstraintIndex);
				if (!JointSettings.bProjectionEnabled)
				{
					continue;
				}

				Solver.InitProjection(Dt, Settings, JointSettings);
			}
		}

		template<>
		void TPBDJointContainerSolver<FPBDJointSolver>::PreApplyLinearProjectionConstraints(const FReal Dt, const int32 BeginIndex, const int32 EndIndex)
		{
		}

		template<>
		void TPBDJointContainerSolver<FPBDJointCachedSolver>::PreApplyNonLinearProjectionConstraints(const FReal Dt, const int32 BeginIndex, const int32 EndIndex)
		{
		}

		template<>
		void TPBDJointContainerSolver<FPBDJointSolver>::PreApplyNonLinearProjectionConstraints(const FReal Dt, const int32 BeginIndex, const int32 EndIndex)
		{
		}


		template<>
		void TPBDJointContainerSolver<FPBDJointCachedSolver>::ApplyLinearProjectionConstraints(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex)
		{
			constexpr bool bUseSolveLinear = true;
			const FPBDJointSolverSettings& Settings = GetSettings();

			for (int32 SolverConstraintIndex = FMath::Max(0, BeginIndex); SolverConstraintIndex < FMath::Min(EndIndex, ConstraintSolvers.Num()); ++SolverConstraintIndex)
			{
				FPBDJointCachedSolver& Solver = ConstraintSolvers[SolverConstraintIndex];
				if (!Solver.RequiresSolve())
				{
					continue;
				}

				const FPBDJointSettings& JointSettings = GetConstraintSettings(SolverConstraintIndex);
				if (!JointSettings.bProjectionEnabled)
				{
					continue;
				}

				if (It == 0)
				{
					Solver.ApplyTeleports(Dt, Settings, JointSettings);
				}

				const bool bLastIteration = (It == (NumIts - 1));
				Solver.ApplyProjections(Dt, Settings, JointSettings, bLastIteration);
			}
		}

		template<>
		void TPBDJointContainerSolver<FPBDJointSolver>::ApplyLinearProjectionConstraints(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex)
		{
		}

		template<>
		void TPBDJointContainerSolver<FPBDJointCachedSolver>::ApplyNonLinearProjectionConstraints(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex)
		{
		}

		template<>
		void TPBDJointContainerSolver<FPBDJointSolver>::ApplyNonLinearProjectionConstraints(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex)
		{
			const FPBDJointSolverSettings& Settings = GetSettings();

			for (int32 SolverConstraintIndex = FMath::Max(0, BeginIndex); SolverConstraintIndex < FMath::Min(EndIndex, ConstraintSolvers.Num()); ++SolverConstraintIndex)
			{
				FPBDJointSolver& Solver = ConstraintSolvers[SolverConstraintIndex];
				if (!Solver.RequiresSolve())
				{
					continue;
				}

				const FPBDJointSettings& JointSettings = GetConstraintSettings(SolverConstraintIndex);
				if (!JointSettings.bProjectionEnabled)
				{
					continue;
				}

				Solver.Update(Dt, Settings, JointSettings);

				if (It == 0)
				{
					// @todo(chaos): support reverse parent/child
					Solver.Body1().UpdateRotationDependentState();
					Solver.UpdateMasses(FReal(0), FReal(1));
				}

				const bool bLastIteration = (It == (NumIts - 1));
				Solver.ApplyProjections(Dt, Settings, JointSettings, bLastIteration);
			}
		}
	}	// namespace Private
}	// namespace Chaos

template class Chaos::Private::TPBDJointContainerSolver<Chaos::FPBDJointCachedSolver>;
template class Chaos::Private::TPBDJointContainerSolver<Chaos::FPBDJointSolver>;
