// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/Joint/PBDJointSolverGaussSeidel.h"
#include "Chaos/Joint/ChaosJointLog.h"
#include "Chaos/Joint/JointConstraintsCVars.h"
#include "Chaos/Joint/JointSolverConstraints.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/PBDJointConstraintUtilities.h"
#include "Chaos/Utilities.h"
#include "ChaosStats.h"
#if INTEL_ISPC
#include "PBDJointSolverGaussSeidel.ispc.generated.h"

static_assert(sizeof(ispc::FVector) == sizeof(Chaos::FVec3), "sizeof(ispc::FVector) != sizeof(Chaos::FVec3)");
static_assert(sizeof(ispc::FTransform) == sizeof(Chaos::FRigidTransform3), "sizeof(ispc::FTransform) != sizeof(Chaos::FRigidTransform3)");
static_assert(sizeof(ispc::FVector4) == sizeof(Chaos::FRotation3), "sizeof(ispc::FVector4) != sizeof(Chaos::FRotation3)");
static_assert(sizeof(ispc::FMatrix) == sizeof(Chaos::FMatrix33), "sizeof(ispc::FMatrix) != sizeof(Chaos::FMatrix33)");
#endif


namespace Chaos
{


	//
	//
	//////////////////////////////////////////////////////////////////////////
	//
	//

	void FPBDJointSolver::InitDerivedState()
	{
		InitConnectorXs[0] = X(0) + R(0) * LocalConnectorXs[0].GetTranslation();
		InitConnectorXs[1] = X(1) + R(1) * LocalConnectorXs[1].GetTranslation();
		InitConnectorRs[0] = R(0) * LocalConnectorXs[0].GetRotation();
		InitConnectorRs[1] = R(1) * LocalConnectorXs[1].GetRotation();
		InitConnectorRs[1].EnforceShortestArcWith(InitConnectorRs[0]);

		const FVec3 BodyP0 = P(0);
		const FRotation3 BodyQ0 = Q(0);
		const FVec3 BodyP1 = P(1);
		const FRotation3 BodyQ1 = Q(1);
		ConnectorXs[0] = BodyP0 + BodyQ0 * LocalConnectorXs[0].GetTranslation();
		ConnectorXs[1] = BodyP1 + BodyQ1 * LocalConnectorXs[1].GetTranslation();
		ConnectorRs[0] = BodyQ0 * LocalConnectorXs[0].GetRotation();
		ConnectorRs[1] = BodyQ1 * LocalConnectorXs[1].GetRotation();
		ConnectorRs[1].EnforceShortestArcWith(ConnectorRs[0]);
	}


	void FPBDJointSolver::UpdateDerivedState()
	{
		// Kinematic bodies will not be moved, so we don't update derived state during iterations
		if (InvM(0) > UE_SMALL_NUMBER)
		{
			const FVec3 BodyP0 = P(0);
			const FRotation3 BodyQ0 = Q(0);
			ConnectorXs[0] = BodyP0 + BodyQ0 * LocalConnectorXs[0].GetTranslation();
			ConnectorRs[0] = BodyQ0 * LocalConnectorXs[0].GetRotation();
		}
		if (InvM(1) > UE_SMALL_NUMBER)
		{
			const FVec3 BodyP1 = P(1);
			const FRotation3 BodyQ1 = Q(1);
			ConnectorXs[1] = BodyP1 + BodyQ1 * LocalConnectorXs[1].GetTranslation();
			ConnectorRs[1] = BodyQ1 * LocalConnectorXs[1].GetRotation();
		}
		ConnectorRs[1].EnforceShortestArcWith(ConnectorRs[0]);
	}


	void FPBDJointSolver::UpdateDerivedState(const int32 BodyIndex)
	{
		const FVec3 BodyP = P(BodyIndex);
		const FRotation3 BodyQ = Q(BodyIndex);
		ConnectorXs[BodyIndex] = BodyP + BodyQ * LocalConnectorXs[BodyIndex].GetTranslation();
		ConnectorRs[BodyIndex] = BodyQ * LocalConnectorXs[BodyIndex].GetRotation();
		ConnectorRs[1].EnforceShortestArcWith(ConnectorRs[0]);
	}

	void FPBDJointSolver::Init(
		const FReal Dt,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings,
		const FRigidTransform3& XL0,
		const FRigidTransform3& XL1)
	{
		LocalConnectorXs[0] = XL0;
		LocalConnectorXs[1] = XL1;

		// \todo(chaos): joint should support parent/child in either order
		SolverBodies[0].SetInvMScale(JointSettings.ParentInvMassScale);
		SolverBodies[1].SetInvMScale(FReal(1));
		SolverBodies[0].SetInvIScale(JointSettings.ParentInvMassScale);
		SolverBodies[1].SetInvIScale(FReal(1));
		SolverBodies[0].SetShockPropagationScale(FReal(1));
		SolverBodies[1].SetShockPropagationScale(FReal(1));

		// Set the mass and inertia.
		// If enabled, adjust the mass so that we limit the maximum mass and inertia ratios
		InvMScales[0] = FReal(1);
		InvMScales[1] = FReal(1);
		ConditionedInvMs[0] = Body0().InvM();
		ConditionedInvMs[1] = Body1().InvM();
		ConditionedInvILs[0] = Body0().InvILocal();
		ConditionedInvILs[1] = Body1().InvILocal();
		if (JointSettings.bMassConditioningEnabled)
		{
			FPBDJointUtilities::ConditionInverseMassAndInertia(Body0().InvM(), Body1().InvM(), Body0().InvILocal(), Body1().InvILocal(), SolverSettings.MinParentMassRatio, SolverSettings.MaxInertiaRatio, ConditionedInvMs[0], ConditionedInvMs[1], ConditionedInvILs[0], ConditionedInvILs[1]);
		}
		UpdateMass0();
		UpdateMass1();

		NetLinearImpulse = FVec3(0);
		NetAngularImpulse = FVec3(0);

		LinearViolationSq = 0;
		AngularViolation = 0;

		LinearSoftLambda = 0;
		TwistSoftLambda = 0;
		SwingSoftLambda = 0;
		LinearDriveLambdas = FVec3(0);
		RotationDriveLambdas = FVec3(0);

		// Tolerances are positional errors below visible detection. But in PBD the errors
		// we leave behind get converted to velocity, so we need to ensure that the resultant
		// movement from that erroneous velocity is less than the desired position tolerance.
		// Assume that the tolerances were defined for a 60Hz simulation, then it must be that
		// the position error is less than the position change from constant external forces
		// (e.g., gravity). So, we are saying that the tolerance was chosen because the position
		// error is less that F.dt^2. We need to scale the tolerance to work at our current dt.
		const FReal ToleranceScale = FMath::Min(1.f, 60.f * 60.f * Dt * Dt);
		PositionTolerance = ToleranceScale * SolverSettings.PositionTolerance;
		AngleTolerance = ToleranceScale * SolverSettings.AngleTolerance;

		SolverStiffness = 1.0f;
		bIsBroken = false;

		LinearHardLambda = FVec3(0);
		AngularHardLambda = FVec3(0);

		InitDerivedState();

		if (JointSettings.LinearRestitution != 0.0f)
		{
			CalculateConstraintAxisLinearVelocities(JointSettings, InitConstraintAxisLinearVelocities);
		}
		if (JointSettings.TwistRestitution != 0.0f || JointSettings.SwingRestitution != 0.0f)
		{
			CalculateConstraintAxisAngularVelocities(SolverSettings, JointSettings, InitConstraintAxisAngularVelocities);
		}
	}


	void FPBDJointSolver::Deinit()
	{
		SolverBodies[0].Reset();
		SolverBodies[1].Reset();
	}

	void FPBDJointSolver::Update(
		const FReal Dt,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		UpdateDerivedState();
	}

	void FPBDJointSolver::UpdateMasses(
		const FReal InvMScale0,
		const FReal InvMScale1)
	{
		InvMScales[0] = InvMScale0;
		InvMScales[1] = InvMScale1;
		UpdateMass0();
		UpdateMass1();
	}

	void FPBDJointSolver::UpdateMass0()
	{
		if ((ConditionedInvMs[0] > 0) && (InvMScales[0] > 0))
		{
			InvMs[0] = InvMScales[0] * ConditionedInvMs[0];
			InvIs[0] = Utilities::ComputeWorldSpaceInertia(Q(0), InvMScales[0] * ConditionedInvILs[0]);
		}
		else
		{
			InvMs[0] = 0;
			InvIs[0] = FMatrix33(0);
		}
	}

	void FPBDJointSolver::UpdateMass1()
	{
		if ((ConditionedInvMs[1] > 0) && (InvMScales[1] > 0))
		{
			InvMs[1] = InvMScales[1] * ConditionedInvMs[1];
			InvIs[1] = Utilities::ComputeWorldSpaceInertia(Q(1), InvMScales[1] * ConditionedInvILs[1]);
		}
		else
		{
			InvMs[1] = 0;
			InvIs[1] = FMatrix33(0);
		}
	}

	void FPBDJointSolver::SetShockPropagationScales(const FReal InvMScale0, const FReal InvMScale1, const FReal Dt)
	{
		if (InvMScales[0] != InvMScale0)
		{
			InvMScales[0] = InvMScale0;
			UpdateMass0();
		}
		if (InvMScales[1] != InvMScale1)
		{
			InvMScales[1] = InvMScale1;
			UpdateMass1();
		}
	}

	void FPBDJointSolver::ApplyConstraints(
		const FReal Dt,
		const FReal InSolverStiffness,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		SolverStiffness = InSolverStiffness;

		LinearViolationSq = 0.f;
		AngularViolation = 0.f;

		if (SolverSettings.bSolvePositionLast)
		{
			ApplyRotationConstraints(Dt, SolverSettings, JointSettings);
			ApplyPositionConstraints(Dt, SolverSettings, JointSettings);

			ApplyRotationDrives(Dt, SolverSettings, JointSettings);
			ApplyPositionDrives(Dt, SolverSettings, JointSettings);
		}
		else
		{
			ApplyPositionConstraints(Dt, SolverSettings, JointSettings);
			ApplyRotationConstraints(Dt, SolverSettings, JointSettings);

			ApplyPositionDrives(Dt, SolverSettings, JointSettings);
			ApplyRotationDrives(Dt, SolverSettings, JointSettings);
		}
	}


	void FPBDJointSolver::ApplyVelocityConstraints(
		const FReal Dt,
		const FReal InSolverStiffness,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		SolverStiffness = InSolverStiffness;

		// This is used for the QuasiPbd solver. If the Pbd step applied impulses to
		// correct position errors, it will have introduced a velocity equal to the 
		// correction divided by the timestep. We ensure that the velocity constraints
		// (including restitution) are also enforced. This also prevents any position
		// errors from the previous frame getting converted into energy.

		if (SolverSettings.bSolvePositionLast)
		{
			ApplyAngularVelocityConstraints(Dt, SolverSettings, JointSettings);
			ApplyLinearVelocityConstraints(Dt, SolverSettings, JointSettings);
		}
		else
		{
			ApplyLinearVelocityConstraints(Dt, SolverSettings, JointSettings);
			ApplyAngularVelocityConstraints(Dt, SolverSettings, JointSettings);
		}

		// @todo(chaos): We can also apply velocity drives here rather than in the Pbd pass
	}

	void FPBDJointSolver::ApplyPositionProjection(
		const FReal Dt,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings,
		FVec3& DP1,
		FVec3& DR1)
	{
		// Position Projection
		const FReal LinearProjection = FPBDJointUtilities::GetLinearProjection(SolverSettings, JointSettings);
		const bool bLinearSoft = FPBDJointUtilities::GetSoftLinearLimitEnabled(SolverSettings, JointSettings);
		const bool bLinearProjectionEnabled = (!bLinearSoft && JointSettings.bProjectionEnabled);
		const TVec3<EJointMotionType>& LinearMotion = JointSettings.LinearMotionTypes;
		const bool bLinearLocked = (LinearMotion[0] == EJointMotionType::Locked) && (LinearMotion[1] == EJointMotionType::Locked) && (LinearMotion[2] == EJointMotionType::Locked);
		const bool bLinearLimited = (LinearMotion[0] == EJointMotionType::Limited) && (LinearMotion[1] == EJointMotionType::Limited) && (LinearMotion[2] == EJointMotionType::Limited);
		if (bLinearProjectionEnabled && (LinearProjection > 0))
		{
			if (bLinearLocked)
			{
				ApplyPointProjection(Dt, SolverSettings, JointSettings, LinearProjection, DP1, DR1);
			}
			else if (bLinearLimited)
			{
				ApplySphereProjection(Dt, SolverSettings, JointSettings, LinearProjection, DP1, DR1);
			}
			// @todo(ccaulfield): support mixed linear projection
		}
	}

	void FPBDJointSolver::ApplyRotationProjection(
		const FReal Dt,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings,
		FVec3& DP1,
		FVec3& DR1)
	{
		const FReal AngularProjection = FPBDJointUtilities::GetAngularProjection(SolverSettings, JointSettings);
		const TVec3<EJointMotionType>& LinearMotion = JointSettings.LinearMotionTypes;
		const bool bLinearLocked = (LinearMotion[0] == EJointMotionType::Locked) && (LinearMotion[1] == EJointMotionType::Locked) && (LinearMotion[2] == EJointMotionType::Locked);

		// Twist projection
		const bool bTwistSoft = FPBDJointUtilities::GetSoftTwistLimitEnabled(SolverSettings, JointSettings);
		const bool bTwistProjectionEnabled = SolverSettings.bEnableTwistLimits && (!bTwistSoft && JointSettings.bProjectionEnabled);
		if (bTwistProjectionEnabled && (AngularProjection > 0.0f))
		{
			const EJointMotionType TwistMotion = JointSettings.AngularMotionTypes[(int32)EJointAngularConstraintIndex::Twist];
			if (TwistMotion != EJointMotionType::Free)
			{
				ApplyTwistProjection(Dt, SolverSettings, JointSettings, AngularProjection, bLinearLocked, DP1, DR1);
			}
		}

		// Swing projection
		const bool bSwingSoft = FPBDJointUtilities::GetSoftSwingLimitEnabled(SolverSettings, JointSettings);
		const bool bSwingProjectionEnabled = SolverSettings.bEnableSwingLimits && (!bSwingSoft && JointSettings.bProjectionEnabled);
		if (bSwingProjectionEnabled && (AngularProjection > 0.0f))
		{
			const EJointMotionType Swing1Motion = JointSettings.AngularMotionTypes[(int32)EJointAngularConstraintIndex::Swing1];
			const EJointMotionType Swing2Motion = JointSettings.AngularMotionTypes[(int32)EJointAngularConstraintIndex::Swing2];
			if ((Swing1Motion == EJointMotionType::Limited) && (Swing2Motion == EJointMotionType::Limited))
			{
				ApplyConeProjection(Dt, SolverSettings, JointSettings, AngularProjection, bLinearLocked, DP1, DR1);
			}
			else if ((Swing1Motion == EJointMotionType::Limited) && (Swing2Motion == EJointMotionType::Locked))
			{
				ApplySingleLockedSwingProjection(Dt, SolverSettings, JointSettings, EJointAngularConstraintIndex::Swing2, AngularProjection, bLinearLocked, DP1, DR1);
				ApplySwingProjection(Dt, SolverSettings, JointSettings, EJointAngularConstraintIndex::Swing1, AngularProjection, bLinearLocked, DP1, DR1);
			}
			else if ((Swing1Motion == EJointMotionType::Limited) && (Swing2Motion == EJointMotionType::Free))
			{
				ApplyDualConeSwingProjection(Dt, SolverSettings, JointSettings, EJointAngularConstraintIndex::Swing1, AngularProjection, bLinearLocked, DP1, DR1);
			}
			else if ((Swing1Motion == EJointMotionType::Locked) && (Swing2Motion == EJointMotionType::Limited))
			{
				ApplySingleLockedSwingProjection(Dt, SolverSettings, JointSettings, EJointAngularConstraintIndex::Swing1, AngularProjection, bLinearLocked, DP1, DR1);
				ApplySwingProjection(Dt, SolverSettings, JointSettings, EJointAngularConstraintIndex::Swing2, AngularProjection, bLinearLocked, DP1, DR1);
			}
			else if ((Swing1Motion == EJointMotionType::Locked) && (Swing2Motion == EJointMotionType::Locked))
			{
				ApplyDoubleLockedSwingProjection(Dt, SolverSettings, JointSettings, AngularProjection, bLinearLocked, DP1, DR1);
			}
			else if ((Swing1Motion == EJointMotionType::Locked) && (Swing2Motion == EJointMotionType::Free))
			{
				ApplySingleLockedSwingProjection(Dt, SolverSettings, JointSettings, EJointAngularConstraintIndex::Swing1, AngularProjection, bLinearLocked, DP1, DR1);
			}
			else if ((Swing1Motion == EJointMotionType::Free) && (Swing2Motion == EJointMotionType::Limited))
			{
				ApplyDualConeSwingProjection(Dt, SolverSettings, JointSettings, EJointAngularConstraintIndex::Swing2, AngularProjection, bLinearLocked, DP1, DR1);
			}
			else if ((Swing1Motion == EJointMotionType::Free) && (Swing2Motion == EJointMotionType::Locked))
			{
				ApplySingleLockedSwingProjection(Dt, SolverSettings, JointSettings, EJointAngularConstraintIndex::Swing2, AngularProjection, bLinearLocked, DP1, DR1);
			}
		}
	}

	void FPBDJointSolver::ApplyProjections(
		const FReal Dt,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings,
		const bool bLastIteration)
	{
		// @todo(chaos): We need to handle parent/child being the other way round
		if (!IsDynamic(1))
		{
			// If child is kinematic, return. 
			return;
		}

		SolverStiffness = 1;


		FVec3 DP1 = FVec3(0);
		FVec3 DR1 = FVec3(0);

		if (SolverSettings.bSolvePositionLast)
		{
			ApplyRotationProjection(Dt, SolverSettings, JointSettings, DP1, DR1);
			ApplyPositionProjection(Dt, SolverSettings, JointSettings, DP1, DR1);
		}
		else
		{
			ApplyPositionProjection(Dt, SolverSettings, JointSettings, DP1, DR1);
			ApplyRotationProjection(Dt, SolverSettings, JointSettings, DP1, DR1);
		}

		// Final position fixup
		if (bLastIteration)
		{
			const TVec3<EJointMotionType>& LinearMotion = JointSettings.LinearMotionTypes;
			const bool bLinearLocked = (LinearMotion[0] == EJointMotionType::Locked) && (LinearMotion[1] == EJointMotionType::Locked) && (LinearMotion[2] == EJointMotionType::Locked);
			if (bLinearLocked)
			{
				const FReal LinearProjection = FPBDJointUtilities::GetLinearProjection(SolverSettings, JointSettings);
				const bool bLinearSoft = FPBDJointUtilities::GetSoftLinearLimitEnabled(SolverSettings, JointSettings);
				const bool bLinearProjectionEnabled = (!bLinearSoft && JointSettings.bProjectionEnabled);
				if (bLinearProjectionEnabled && (LinearProjection > 0))
				{
					ApplyTranslateProjection(Dt, SolverSettings, JointSettings, LinearProjection, DP1, DR1);
				}

				// Add velocity correction from the net projection motion
				if (Chaos_Joint_VelProjectionAlpha > 0.0f)
				{
					ApplyVelocityProjection(Dt, SolverSettings, JointSettings, Chaos_Joint_VelProjectionAlpha, DP1, DR1);
				}
			}
		}
	}


	void FPBDJointSolver::ApplyRotationConstraints(
		const FReal Dt,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		bool bHasRotationConstraints =
			(JointSettings.AngularMotionTypes[0] != EJointMotionType::Free)
			|| (JointSettings.AngularMotionTypes[1] != EJointMotionType::Free)
			|| (JointSettings.AngularMotionTypes[2] != EJointMotionType::Free);
		if (!bHasRotationConstraints)
		{
			return;
		}

		// Locked axes always use hard constraints. Limited axes use hard or soft depending on settings
		EJointMotionType TwistMotion = JointSettings.AngularMotionTypes[(int32)EJointAngularConstraintIndex::Twist];
		EJointMotionType Swing1Motion = JointSettings.AngularMotionTypes[(int32)EJointAngularConstraintIndex::Swing1];
		EJointMotionType Swing2Motion = JointSettings.AngularMotionTypes[(int32)EJointAngularConstraintIndex::Swing2];
		bool bTwistSoft = FPBDJointUtilities::GetSoftTwistLimitEnabled(SolverSettings, JointSettings);
		bool bSwingSoft = FPBDJointUtilities::GetSoftSwingLimitEnabled(SolverSettings, JointSettings);

		// If the twist axes are opposing, we cannot decompose the orientation into swing and twist angles, so just give up
		const FVec3 Twist0 = ConnectorRs[0] * FJointConstants::TwistAxis();
		const FVec3 Twist1 = ConnectorRs[1] * FJointConstants::TwistAxis();
		const FReal Twist01Dot = FVec3::DotProduct(Twist0, Twist1);
		const bool bDegenerate = (Twist01Dot < Chaos_Joint_DegenerateRotationLimit);
		if (bDegenerate)
		{
			UE_LOGF(LogChaosJoint, VeryVerbose, " Degenerate rotation at Swing %f deg", FMath::RadiansToDegrees(FMath::Acos(Twist01Dot)));
		}

		// Apply twist constraint
		// NOTE: Cannot calculate twist angle at 180degree swing
		if (SolverSettings.bEnableTwistLimits && !bDegenerate)
		{
			if (TwistMotion == EJointMotionType::Limited)
			{
				ApplyTwistConstraint(Dt, SolverSettings, JointSettings, bTwistSoft);
			}
			else if (TwistMotion == EJointMotionType::Locked)
			{
				// Covered below
			}
			else if (TwistMotion == EJointMotionType::Free)
			{
			}
		}

		// Apply swing constraints
		// NOTE: Cannot separate swing angles at 180degree swing (but we can still apply locks)
		if (SolverSettings.bEnableSwingLimits)
		{
			if ((Swing1Motion == EJointMotionType::Limited) && (Swing2Motion == EJointMotionType::Limited))
			{
				ApplyConeConstraint(Dt, SolverSettings, JointSettings, bSwingSoft);
			}
			else if ((Swing1Motion == EJointMotionType::Limited) && (Swing2Motion == EJointMotionType::Locked))
			{
				ApplySingleLockedSwingConstraint(Dt, SolverSettings, JointSettings, EJointAngularConstraintIndex::Swing2, false);
				if (!bDegenerate)
				{
					ApplySwingConstraint(Dt, SolverSettings, JointSettings, EJointAngularConstraintIndex::Swing1, bSwingSoft);
				}
			}
			else if ((Swing1Motion == EJointMotionType::Limited) && (Swing2Motion == EJointMotionType::Free))
			{
				if (!bDegenerate)
				{
					ApplyDualConeSwingConstraint(Dt, SolverSettings, JointSettings, EJointAngularConstraintIndex::Swing1, bSwingSoft);
				}
			}
			else if ((Swing1Motion == EJointMotionType::Locked) && (Swing2Motion == EJointMotionType::Limited))
			{
				ApplySingleLockedSwingConstraint(Dt, SolverSettings, JointSettings, EJointAngularConstraintIndex::Swing1, false);
				if (!bDegenerate)
				{
					ApplySwingConstraint(Dt, SolverSettings, JointSettings, EJointAngularConstraintIndex::Swing2, bSwingSoft);
				}
			}
			else if ((Swing1Motion == EJointMotionType::Locked) && (Swing2Motion == EJointMotionType::Locked))
			{
				// Covered below
			}
			else if ((Swing1Motion == EJointMotionType::Locked) && (Swing2Motion == EJointMotionType::Free))
			{
				ApplySingleLockedSwingConstraint(Dt, SolverSettings, JointSettings, EJointAngularConstraintIndex::Swing1, false);
			}
			else if ((Swing1Motion == EJointMotionType::Free) && (Swing2Motion == EJointMotionType::Limited))
			{
				if (!bDegenerate)
				{
					ApplyDualConeSwingConstraint(Dt, SolverSettings, JointSettings, EJointAngularConstraintIndex::Swing2, bSwingSoft);
				}
			}
			else if ((Swing1Motion == EJointMotionType::Free) && (Swing2Motion == EJointMotionType::Locked))
			{
				ApplySingleLockedSwingConstraint(Dt, SolverSettings, JointSettings, EJointAngularConstraintIndex::Swing2, false);
			}
			else if ((Swing1Motion == EJointMotionType::Free) && (Swing2Motion == EJointMotionType::Free))
			{
			}
		}

		// Note: single-swing locks are already handled above so we only need to do something here if both are locked
		bool bLockedTwist = SolverSettings.bEnableTwistLimits 
			&& (JointSettings.AngularMotionTypes[(int32)EJointAngularConstraintIndex::Twist] == EJointMotionType::Locked);
		bool bLockedSwing = SolverSettings.bEnableSwingLimits 
			&& (JointSettings.AngularMotionTypes[(int32)EJointAngularConstraintIndex::Swing1] == EJointMotionType::Locked) 
			&& (JointSettings.AngularMotionTypes[(int32)EJointAngularConstraintIndex::Swing2] == EJointMotionType::Locked);
		if (bLockedTwist || bLockedSwing)
		{
			ApplyLockedRotationConstraints(Dt, SolverSettings, JointSettings, bLockedTwist, bLockedSwing);
		}
	}


	void FPBDJointSolver::ApplyRotationDrives(
		const FReal Dt,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		bool bHasRotationDrives =
			JointSettings.bAngularTwistPositionDriveEnabled
			|| JointSettings.bAngularTwistVelocityDriveEnabled
			|| JointSettings.bAngularSwingPositionDriveEnabled
			|| JointSettings.bAngularSwingVelocityDriveEnabled
			|| JointSettings.bAngularSLerpPositionDriveEnabled
			|| JointSettings.bAngularSLerpVelocityDriveEnabled;
		if (!bHasRotationDrives)
		{
			return;
		}

		EJointMotionType TwistMotion = JointSettings.AngularMotionTypes[(int32)EJointAngularConstraintIndex::Twist];
		EJointMotionType Swing1Motion = JointSettings.AngularMotionTypes[(int32)EJointAngularConstraintIndex::Swing1];
		EJointMotionType Swing2Motion = JointSettings.AngularMotionTypes[(int32)EJointAngularConstraintIndex::Swing2];

		if (SolverSettings.bEnableDrives)
		{
			bool bTwistLocked = TwistMotion == EJointMotionType::Locked;
			bool bSwing1Locked = Swing1Motion == EJointMotionType::Locked;
			bool bSwing2Locked = Swing2Motion == EJointMotionType::Locked;

			// No SLerp drive if we have a locked rotation (it will be grayed out in the editor in this case, but could still have been set before the rotation was locked)
			// @todo(ccaulfield): setting should be cleaned up before being passed to the solver
			if ((JointSettings.bAngularSLerpPositionDriveEnabled || JointSettings.bAngularSLerpVelocityDriveEnabled) && !bTwistLocked && !bSwing1Locked && !bSwing2Locked)
			{
				ApplySLerpDrive(Dt, SolverSettings, JointSettings);
			}
			else
			{
				const bool bTwistDriveEnabled = ((JointSettings.bAngularTwistPositionDriveEnabled || JointSettings.bAngularTwistVelocityDriveEnabled) && !bTwistLocked);
				const bool bSwingDriveEnabled = (JointSettings.bAngularSwingPositionDriveEnabled || JointSettings.bAngularSwingVelocityDriveEnabled);
				const bool bSwing1DriveEnabled = bSwingDriveEnabled && !bSwing1Locked;
				const bool bSwing2DriveEnabled = bSwingDriveEnabled && !bSwing2Locked;
				if (bTwistDriveEnabled || bSwing1DriveEnabled || bSwing2DriveEnabled)
				{
					ApplySwingTwistDrives(Dt, SolverSettings, JointSettings, bTwistDriveEnabled, bSwing1DriveEnabled, bSwing2DriveEnabled);
				}
			}
		}
	}


	void FPBDJointSolver::ApplyPositionConstraints(
		const FReal Dt,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		bool bHasPositionConstraints =
			(JointSettings.LinearMotionTypes[0] != EJointMotionType::Free)
			|| (JointSettings.LinearMotionTypes[1] != EJointMotionType::Free)
			|| (JointSettings.LinearMotionTypes[2] != EJointMotionType::Free);
		if (!bHasPositionConstraints)
		{
			return;
		}

		const TVec3<EJointMotionType>& LinearMotion = JointSettings.LinearMotionTypes;
		const TVec3<bool> bLinearLocked =
		{
			(LinearMotion[0] == EJointMotionType::Locked),
			(LinearMotion[1] == EJointMotionType::Locked),
			(LinearMotion[2] == EJointMotionType::Locked),
		};
		const TVec3<bool> bLinearLimited =
		{
			(LinearMotion[0] == EJointMotionType::Limited),
			(LinearMotion[1] == EJointMotionType::Limited),
			(LinearMotion[2] == EJointMotionType::Limited),
		};

		if (bLinearLocked[0] && bLinearLocked[1] && bLinearLocked[2])
		{
			// Hard point constraint (most common case)
			if (!IsDynamic(0))
			{
				ApplyPointPositionConstraintKD(0, 1, Dt, SolverSettings, JointSettings);
			}
			else if (!IsDynamic(1))
			{
				ApplyPointPositionConstraintKD(1, 0, Dt, SolverSettings, JointSettings);
			}
			else
			{
				ApplyPointPositionConstraintDD(Dt, SolverSettings, JointSettings);
			}
		}
		else if (bLinearLimited[0] && bLinearLimited[1] && bLinearLimited[2])
		{
			// Spherical constraint
			ApplySphericalPositionConstraint(Dt, SolverSettings, JointSettings);
		}
		else if (bLinearLocked[1] && bLinearLocked[2] && !bLinearLocked[0])
		{
			// Line constraint along X axis
			ApplyCylindricalPositionConstraint(Dt, 0, LinearMotion[0], EJointMotionType::Locked, SolverSettings, JointSettings);
		}
		else if (bLinearLocked[0] && bLinearLocked[2] && !bLinearLocked[1])
		{
			// Line constraint along Y axis
			ApplyCylindricalPositionConstraint(Dt, 1, LinearMotion[1], EJointMotionType::Locked, SolverSettings, JointSettings);
		}
		else if (bLinearLocked[0] && bLinearLocked[1] && !bLinearLocked[2])
		{
			// Line constraint along Z axis
			ApplyCylindricalPositionConstraint(Dt, 2, LinearMotion[2], EJointMotionType::Locked, SolverSettings, JointSettings);
		}
		else if (bLinearLimited[1] && bLinearLimited[2] && !bLinearLimited[0])
		{
			// Cylindrical constraint along X axis
			ApplyCylindricalPositionConstraint(Dt, 0, LinearMotion[0], EJointMotionType::Limited, SolverSettings, JointSettings);
		}
		else if (bLinearLimited[0] && bLinearLimited[2] && !bLinearLimited[1])
		{
			// Cylindrical constraint along Y axis
			ApplyCylindricalPositionConstraint(Dt, 1, LinearMotion[1], EJointMotionType::Limited, SolverSettings, JointSettings);
		}
		else if (bLinearLimited[0] && bLinearLimited[1] && !bLinearLimited[2])
		{
			// Cylindrical constraint along Z axis
			ApplyCylindricalPositionConstraint(Dt, 2, LinearMotion[2], EJointMotionType::Limited, SolverSettings, JointSettings);
		}
		else if (bLinearLocked[0] || bLinearLimited[0])
		{
			// Planar constraint along X axis
			ApplyPlanarPositionConstraint(Dt, 0, LinearMotion[0], SolverSettings, JointSettings);
		}
		else if (bLinearLocked[1] || bLinearLimited[1])
		{
			// Planar constraint along Y axis
			ApplyPlanarPositionConstraint(Dt, 1, LinearMotion[1], SolverSettings, JointSettings);
		}
		else if (bLinearLocked[2] || bLinearLimited[2])
		{
			// Planar constraint along Z axis
			ApplyPlanarPositionConstraint(Dt, 2, LinearMotion[2], SolverSettings, JointSettings);
		}
	}


	void FPBDJointSolver::ApplyPositionDrives(
		const FReal Dt,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		if (SolverSettings.bEnableDrives)
		{
			TVec3<bool> bDriven =
			{
				(JointSettings.bLinearPositionDriveEnabled[0] || JointSettings.bLinearVelocityDriveEnabled[0]) && (JointSettings.LinearMotionTypes[0] != EJointMotionType::Locked),
				(JointSettings.bLinearPositionDriveEnabled[1] || JointSettings.bLinearVelocityDriveEnabled[1]) && (JointSettings.LinearMotionTypes[1] != EJointMotionType::Locked),
				(JointSettings.bLinearPositionDriveEnabled[2] || JointSettings.bLinearVelocityDriveEnabled[2]) && (JointSettings.LinearMotionTypes[2] != EJointMotionType::Locked),
			};

			// Rectangular position drives
			if (bDriven[0] || bDriven[1] || bDriven[2])
			{
				const FMatrix33 R0M = ConnectorRs[0].ToMatrix();
				const FVec3 XTarget = ConnectorXs[0] + ConnectorRs[0] * JointSettings.LinearDrivePositionTarget;
				const FVec3 VTarget = ConnectorRs[0] * JointSettings.LinearDriveVelocityTarget;
				const FVec3 CX = ConnectorXs[1] - XTarget;

				for (int32 AxisIndex = 0; AxisIndex < 3; ++AxisIndex)
				{
					if (bDriven[AxisIndex])
					{
						const FVec3 Axis = R0M.GetAxis(AxisIndex);
						const FReal DeltaPos = FVec3::DotProduct(CX, Axis);
						const FReal DeltaVel = FVec3::DotProduct(VTarget, Axis);

						ApplyPositionDrive(Dt, AxisIndex, SolverSettings, JointSettings, Axis, DeltaPos, DeltaVel);
					}
				}
			}
		}
	}


	//
	//
	//////////////////////////////////////////////////////////////////////////
	//
	//


	void FPBDJointSolver::ApplyLinearVelocityConstraints(
		const FReal Dt,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		bool bHasPositionConstraints =
			(JointSettings.LinearMotionTypes[0] != EJointMotionType::Free)
			|| (JointSettings.LinearMotionTypes[1] != EJointMotionType::Free)
			|| (JointSettings.LinearMotionTypes[2] != EJointMotionType::Free);
		if (!bHasPositionConstraints)
		{
			return;
		}

		const TVector<EJointMotionType, 3>& LinearMotion = JointSettings.LinearMotionTypes;
		const TVector<bool, 3> bLinearLocked =
		{
			(LinearMotion[0] == EJointMotionType::Locked),
			(LinearMotion[1] == EJointMotionType::Locked),
			(LinearMotion[2] == EJointMotionType::Locked),
		};
		const TVector<bool, 3> bLinearLimited =
		{
			(LinearMotion[0] == EJointMotionType::Limited),
			(LinearMotion[1] == EJointMotionType::Limited),
			(LinearMotion[2] == EJointMotionType::Limited),
		};

		if (bLinearLocked[0] && bLinearLocked[1] && bLinearLocked[2])
		{
			// Hard point constraint (most common case)
			ApplyPointVelocityConstraint(Dt, SolverSettings, JointSettings);
		}
		else if (bLinearLimited[0] && bLinearLimited[1] && bLinearLimited[2])
		{
			ApplySphericalVelocityConstraint(Dt, SolverSettings, JointSettings);
		}
		else if (bLinearLocked[1] && bLinearLocked[2] && !bLinearLocked[0])
		{
			// Line constraint along X axis
			ApplyCylindricalVelocityConstraint(Dt, 0, LinearMotion[0], EJointMotionType::Locked, SolverSettings, JointSettings);
		}
		else if (bLinearLocked[0] && bLinearLocked[2] && !bLinearLocked[1])
		{
			// Line constraint along Y axis
			ApplyCylindricalVelocityConstraint(Dt, 1, LinearMotion[1], EJointMotionType::Locked, SolverSettings, JointSettings);
		}
		else if (bLinearLocked[0] && bLinearLocked[1] && !bLinearLocked[2])
		{
			// Line constraint along Z axis
			ApplyCylindricalVelocityConstraint(Dt, 2, LinearMotion[2], EJointMotionType::Locked, SolverSettings, JointSettings);
		}
		else if (bLinearLimited[1] && bLinearLimited[2] && !bLinearLimited[0])
		{
			// Cylindrical constraint along X axis
			ApplyCylindricalVelocityConstraint(Dt, 0, LinearMotion[0], EJointMotionType::Limited, SolverSettings, JointSettings);
		}
		else if (bLinearLimited[0] && bLinearLimited[2] && !bLinearLimited[1])
		{
			// Cylindrical constraint along Y axis
			ApplyCylindricalVelocityConstraint(Dt, 1, LinearMotion[1], EJointMotionType::Limited, SolverSettings, JointSettings);
		}
		else if (bLinearLimited[0] && bLinearLimited[1] && !bLinearLimited[2])
		{
			// Cylindrical constraint along Z axis
			ApplyCylindricalVelocityConstraint(Dt, 2, LinearMotion[2], EJointMotionType::Limited, SolverSettings, JointSettings);
		}
		else if (bLinearLocked[0] || bLinearLimited[0])
		{
			// Planar constraint along X axis
			ApplyPlanarVelocityConstraint(Dt, 0, LinearMotion[0], SolverSettings, JointSettings);
		}
		else if (bLinearLocked[1] || bLinearLimited[1])
		{
			// Planar constraint along Y axis
			ApplyPlanarVelocityConstraint(Dt, 1, LinearMotion[1], SolverSettings, JointSettings);
		}
		else if (bLinearLocked[2] || bLinearLimited[2])
		{
			// Planar constraint along Z axis
			ApplyPlanarVelocityConstraint(Dt, 2, LinearMotion[2], SolverSettings, JointSettings);
		}
	}

	void FPBDJointSolver::ApplyAngularVelocityConstraints(
		const FReal Dt,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		bool bHasRotationConstraints =
			(JointSettings.AngularMotionTypes[0] != EJointMotionType::Free)
			|| (JointSettings.AngularMotionTypes[1] != EJointMotionType::Free)
			|| (JointSettings.AngularMotionTypes[2] != EJointMotionType::Free);
		if (!bHasRotationConstraints)
		{
			return;
		}

		// Locked axes always use hard constraints. Limited axes use hard or soft depending on settings
		EJointMotionType TwistMotion = JointSettings.AngularMotionTypes[(int32)EJointAngularConstraintIndex::Twist];
		EJointMotionType Swing1Motion = JointSettings.AngularMotionTypes[(int32)EJointAngularConstraintIndex::Swing1];
		EJointMotionType Swing2Motion = JointSettings.AngularMotionTypes[(int32)EJointAngularConstraintIndex::Swing2];
		bool bTwistSoft = FPBDJointUtilities::GetSoftTwistLimitEnabled(SolverSettings, JointSettings);
		bool bSwingSoft = FPBDJointUtilities::GetSoftSwingLimitEnabled(SolverSettings, JointSettings);

		// If the twist axes are opposing, we cannot decompose the orientation into swing and twist angles, so just give up
		const FVec3 Twist0 = ConnectorRs[0] * FJointConstants::TwistAxis();
		const FVec3 Twist1 = ConnectorRs[1] * FJointConstants::TwistAxis();
		const FReal Twist01Dot = FVec3::DotProduct(Twist0, Twist1);
		const bool bDegenerate = (Twist01Dot < Chaos_Joint_DegenerateRotationLimit);
		if (bDegenerate)
		{
			UE_LOGF(LogChaosJoint, VeryVerbose, " Degenerate rotation at Swing %f deg", FMath::RadiansToDegrees(FMath::Acos(Twist01Dot)));
		}

		// Apply twist constraint
		// NOTE: Cannot calculate twist angle at 180degree swing
		if (SolverSettings.bEnableTwistLimits && !bDegenerate)
		{
			if (TwistMotion == EJointMotionType::Limited)
			{
				ApplyTwistVelocityConstraint(Dt, SolverSettings, JointSettings, bTwistSoft);
			}
			else if (TwistMotion == EJointMotionType::Locked)
			{
				// Covered below
			}
			else if (TwistMotion == EJointMotionType::Free)
			{
			}
		}

		// Apply swing constraints
		// NOTE: Cannot separate swing angles at 180degree swing (but we can still apply locks)
		if (SolverSettings.bEnableSwingLimits)
		{
			if ((Swing1Motion == EJointMotionType::Limited) && (Swing2Motion == EJointMotionType::Limited))
			{
				ApplyConeVelocityConstraint(Dt, SolverSettings, JointSettings, bSwingSoft);
			}
			else if ((Swing1Motion == EJointMotionType::Limited) && (Swing2Motion == EJointMotionType::Locked))
			{
				ApplySingleLockedSwingVelocityConstraint(Dt, SolverSettings, JointSettings, EJointAngularConstraintIndex::Swing2, false);
				if (!bDegenerate)
				{
					ApplySwingVelocityConstraint(Dt, SolverSettings, JointSettings, EJointAngularConstraintIndex::Swing1, bSwingSoft);
				}
			}
			else if ((Swing1Motion == EJointMotionType::Limited) && (Swing2Motion == EJointMotionType::Free))
			{
				if (!bDegenerate)
				{
					ApplyDualConeSwingVelocityConstraint(Dt, SolverSettings, JointSettings, EJointAngularConstraintIndex::Swing1, bSwingSoft);
				}
			}
			else if ((Swing1Motion == EJointMotionType::Locked) && (Swing2Motion == EJointMotionType::Limited))
			{
				ApplySingleLockedSwingVelocityConstraint(Dt, SolverSettings, JointSettings, EJointAngularConstraintIndex::Swing1, false);
				if (!bDegenerate)
				{
					ApplySwingVelocityConstraint(Dt, SolverSettings, JointSettings, EJointAngularConstraintIndex::Swing2, bSwingSoft);
				}
			}
			else if ((Swing1Motion == EJointMotionType::Locked) && (Swing2Motion == EJointMotionType::Locked))
			{
				// Covered below
			}
			else if ((Swing1Motion == EJointMotionType::Locked) && (Swing2Motion == EJointMotionType::Free))
			{
				ApplySingleLockedSwingVelocityConstraint(Dt, SolverSettings, JointSettings, EJointAngularConstraintIndex::Swing1, false);
			}
			else if ((Swing1Motion == EJointMotionType::Free) && (Swing2Motion == EJointMotionType::Limited))
			{
				if (!bDegenerate)
				{
					ApplyDualConeSwingVelocityConstraint(Dt, SolverSettings, JointSettings, EJointAngularConstraintIndex::Swing2, bSwingSoft);
				}
			}
			else if ((Swing1Motion == EJointMotionType::Free) && (Swing2Motion == EJointMotionType::Locked))
			{
				ApplySingleLockedSwingVelocityConstraint(Dt, SolverSettings, JointSettings, EJointAngularConstraintIndex::Swing2, false);
			}
			else if ((Swing1Motion == EJointMotionType::Free) && (Swing2Motion == EJointMotionType::Free))
			{
			}
		}

		// Note: single-swing locks are already handled above so we only need to do something here if both are locked
		bool bLockedTwist = SolverSettings.bEnableTwistLimits 
			&& (JointSettings.AngularMotionTypes[(int32)EJointAngularConstraintIndex::Twist] == EJointMotionType::Locked);
		bool bLockedSwing = SolverSettings.bEnableSwingLimits 
			&& (JointSettings.AngularMotionTypes[(int32)EJointAngularConstraintIndex::Swing1] == EJointMotionType::Locked) 
			&& (JointSettings.AngularMotionTypes[(int32)EJointAngularConstraintIndex::Swing2] == EJointMotionType::Locked);
		if (bLockedTwist || bLockedSwing)
		{
			ApplyLockedRotationVelocityConstraints(Dt, SolverSettings, JointSettings, bLockedTwist, bLockedSwing);
		}
	}

	void FPBDJointSolver::ApplyTwistVelocityConstraint(
		const FReal Dt,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings,
		const bool bUseSoftLimit)
	{
		if(!NetAngularImpulse.IsNearlyZero() && !bUseSoftLimit && AngularHardLambda[(int32)EJointAngularConstraintIndex::Twist] > UE_SMALL_NUMBER)
		{
			FVec3 TwistAxis;
			FReal TwistAngle;
			FPBDJointUtilities::GetTwistAxisAngle(ConnectorRs[0], ConnectorRs[1], TwistAxis, TwistAngle);

			if (TwistAngle < 0.0f)
			{
				TwistAxis = -TwistAxis;
			}

			const FReal TwistStiffness = SolverStiffness * FPBDJointUtilities::GetTwistStiffness(SolverSettings, JointSettings);

			FReal TargetVel = 0.0f;
			if (JointSettings.TwistRestitution != 0.0f)
			{
				const FReal InitVel = InitConstraintAxisAngularVelocities[(int32)EJointAngularConstraintIndex::Twist];
				TargetVel = InitVel > Chaos_Joint_AngularVelocityThresholdToApplyRestitution ? -JointSettings.TwistRestitution * InitVel : 0.0f;
			}
			ApplyAngularVelocityConstraint(TwistStiffness, TwistAxis, TargetVel);
		}
	}

	void FPBDJointSolver::ApplyConeVelocityConstraint(
		const FReal Dt,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings,
		const bool bUseSoftLimit)
	{
		if(!NetAngularImpulse.IsNearlyZero() && !bUseSoftLimit && AngularHardLambda[(int32)EJointAngularConstraintIndex::Swing2] > UE_SMALL_NUMBER)
		{
			FVec3 SwingAxisLocal;
			FReal DSwingAngle = 0.0f;
			const FReal Swing1Limit = JointSettings.AngularLimits[(int32)EJointAngularConstraintIndex::Swing1];
			const FReal Swing2Limit = JointSettings.AngularLimits[(int32)EJointAngularConstraintIndex::Swing2];
			FPBDJointUtilities::GetEllipticalConeAxisErrorLocal(ConnectorRs[0], ConnectorRs[1], Swing2Limit, Swing1Limit, SwingAxisLocal, DSwingAngle);
			// SwingAxisLocal has the size of Sine(SwingAngle), thus we need to normalize it
			SwingAxisLocal.SafeNormalize();
			const FVec3 SwingAxis = ConnectorRs[0] * SwingAxisLocal;
			const FReal SwingStiffness = SolverStiffness * FPBDJointUtilities::GetSwingStiffness(SolverSettings, JointSettings);
			FReal TargetVel = 0.0f;
			if (JointSettings.SwingRestitution != 0.0f)
			{
				const FReal InitVel = InitConstraintAxisAngularVelocities[(int32)EJointAngularConstraintIndex::Swing1];
				TargetVel = InitVel > Chaos_Joint_AngularVelocityThresholdToApplyRestitution ? -JointSettings.SwingRestitution * InitVel : 0.0f;
			}
			ApplyAngularVelocityConstraint(SwingStiffness, SwingAxis, TargetVel);
		}
	}

	void FPBDJointSolver::ApplySingleLockedSwingVelocityConstraint(
		const FReal Dt,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings,
		const EJointAngularConstraintIndex SwingConstraintIndex,
		const bool bUseSoftLimit)
	{
		if(!NetAngularImpulse.IsNearlyZero() && !bUseSoftLimit && AngularHardLambda[(int32)SwingConstraintIndex] > UE_SMALL_NUMBER)
		{
			FVec3 SwingAxis;
			FReal SwingAngle;
			FPBDJointUtilities::GetLockedSwingAxisAngle(ConnectorRs[0], ConnectorRs[1], SwingConstraintIndex, SwingAxis, SwingAngle);
			// Swing axis has the size of Sine(SwingAngle), thus we need to normalize it
			SwingAxis.SafeNormalize();

			const FReal SwingStiffness = SolverStiffness * FPBDJointUtilities::GetSwingStiffness(SolverSettings, JointSettings);
			ApplyAngularVelocityConstraint(SwingStiffness, SwingAxis);
		}
	}

	void FPBDJointSolver::ApplyDualConeSwingVelocityConstraint(
		const FReal Dt,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings,
		const EJointAngularConstraintIndex SwingConstraintIndex,
		const bool bUseSoftLimit)
	{
		if(!NetAngularImpulse.IsNearlyZero() && !bUseSoftLimit && AngularHardLambda[(int32)SwingConstraintIndex] > UE_SMALL_NUMBER)
		{
			FVec3 SwingAxis;
			FReal SwingAngle;
			FPBDJointUtilities::GetDualConeSwingAxisAngle(ConnectorRs[0], ConnectorRs[1], SwingConstraintIndex, SwingAxis, SwingAngle);

			const FReal SwingStiffness = SolverStiffness * FPBDJointUtilities::GetSwingStiffness(SolverSettings, JointSettings);
			FReal TargetVel = 0.0f;
			if (JointSettings.SwingRestitution != 0.0f)
			{
				const FReal InitVel = InitConstraintAxisAngularVelocities[(int32)SwingConstraintIndex];
				TargetVel = InitVel > Chaos_Joint_AngularVelocityThresholdToApplyRestitution ? -JointSettings.SwingRestitution * InitVel : 0.0f;
			}
			ApplyAngularVelocityConstraint(SwingStiffness, SwingAxis, TargetVel);
		}
	}

	void FPBDJointSolver::ApplySwingVelocityConstraint(
		const FReal Dt,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings,
		const EJointAngularConstraintIndex SwingConstraintIndex,
		const bool bUseSoftLimit)
	{
		if(!NetAngularImpulse.IsNearlyZero() && !bUseSoftLimit && AngularHardLambda[(int32)SwingConstraintIndex] > UE_SMALL_NUMBER)
		{
			FVec3 SwingAxis;
			FReal SwingAngle;
			FPBDJointUtilities::GetSwingAxisAngle(ConnectorRs[0], ConnectorRs[1], SolverSettings.SwingTwistAngleTolerance, SwingConstraintIndex, SwingAxis, SwingAngle);

			const FReal SwingStiffness = SolverStiffness * FPBDJointUtilities::GetSwingStiffness(SolverSettings, JointSettings);
			FReal TargetVel = 0.0f;
			if (JointSettings.SwingRestitution != 0.0f)
			{
				const FReal InitVel = InitConstraintAxisAngularVelocities[(int32)SwingConstraintIndex];
				TargetVel = InitVel > Chaos_Joint_AngularVelocityThresholdToApplyRestitution ? -JointSettings.SwingRestitution * InitVel : 0.0f;
			}
			ApplyAngularVelocityConstraint(SwingStiffness, SwingAxis, TargetVel);
		}
	}

	void FPBDJointSolver::ApplyAngularVelocityConstraint(
		const FReal Stiffness,
		const FVec3& Axis,
		const FReal TargetVel)
	{
		const FVec3 CW = W(1) - W(0);
		const FVec3 IA0 = Utilities::Multiply(InvI(0), Axis);
		const FVec3 IA1 = Utilities::Multiply(InvI(1), Axis);
		const FReal II0 = FVec3::DotProduct(Axis, IA0);
		const FReal II1 = FVec3::DotProduct(Axis, IA1);

		const FVec3 AngularImpulse = Stiffness * (FVec3::DotProduct(CW, Axis) - TargetVel) / (II0 + II1) * Axis;
		const FVec3 DW0 = Utilities::Multiply(InvI(0), AngularImpulse);
		const FVec3 DW1 = Utilities::Multiply(InvI(1), -AngularImpulse);

		ApplyAngularVelocityDelta(DW0, DW1);
	}

	void FPBDJointSolver::ApplyLockedRotationVelocityConstraints(
		const FReal Dt,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings,
		const bool bApplyTwist,
		const bool bApplySwing)
	{
		if(!NetAngularImpulse.IsNearlyZero())
		{
			FVec3 Axis0, Axis1, Axis2;
			FPBDJointUtilities::GetLockedRotationAxes(ConnectorRs[0], ConnectorRs[1], Axis0, Axis1, Axis2);
			const FRotation3 R01 = ConnectorRs[0].Inverse() * ConnectorRs[1];

			if (bApplyTwist && AngularHardLambda[(int32)EJointAngularConstraintIndex::Twist] > UE_SMALL_NUMBER)
			{
				FReal TwistStiffness = SolverStiffness * FPBDJointUtilities::GetTwistStiffness(SolverSettings, JointSettings);
				Axis0.SafeNormalize();
				ApplyAngularVelocityConstraint(TwistStiffness, Axis0);
			}

			if (bApplySwing && AngularHardLambda[(int32)EJointAngularConstraintIndex::Swing2] + AngularHardLambda[(int32)EJointAngularConstraintIndex::Swing1] > UE_SMALL_NUMBER)
			{
				FReal SwingStiffness = SolverStiffness * FPBDJointUtilities::GetSwingStiffness(SolverSettings, JointSettings);
				Axis1.SafeNormalize();
				ApplyAngularVelocityConstraint(SwingStiffness, Axis1);
				Axis2.SafeNormalize();
				ApplyAngularVelocityConstraint(SwingStiffness, Axis2);
			}
		}
	}


	//
	//
	//////////////////////////////////////////////////////////////////////////
	//
	//


	void FPBDJointSolver::ApplyPositionDelta(
		const int32 BodyIndex,
		const FVec3& DP)
	{
		//UE_LOGF(LogChaosJoint, VeryVerbose, "      Apply DP%d %f %f %f", BodyIndex, DP.X, DP.Y, DP.Z);

		Body(BodyIndex).ApplyPositionDelta(DP);

		ConnectorXs[BodyIndex] += DP;
	}


	void FPBDJointSolver::ApplyPositionDelta(
		const FVec3& DP0,
		const FVec3& DP1)
	{
		//UE_LOGF(LogChaosJoint, VeryVerbose, "      Apply DP%d %f %f %f", 0, DP0.X, DP0.Y, DP0.Z);
		//UE_LOGF(LogChaosJoint, VeryVerbose, "      Apply DP%d %f %f %f", 1, DP1.X, DP1.Y, DP1.Z);

		Body(0).ApplyPositionDelta(DP0);
		Body(1).ApplyPositionDelta(DP1);

		ConnectorXs[0] += DP0;
		ConnectorXs[1] += DP1;
	}


	void FPBDJointSolver::ApplyRotationDelta(
		const int32 BodyIndex,
		const FVec3& DR)
	{
		//UE_LOGF(LogChaosJoint, VeryVerbose, "      Apply DR%d %f %f %f", BodyIndex, DR.X, DR.Y, DR.Z);

		Body(BodyIndex).ApplyRotationDelta(DR);

		UpdateDerivedState(BodyIndex);
	}


	void FPBDJointSolver::ApplyRotationDelta(
		const FVec3& DR0,
		const FVec3& DR1)
	{
		//UE_LOGF(LogChaosJoint, VeryVerbose, "      Apply DR%d %f %f %f", 0, DR0.X, DR0.Y, DR0.Z);
		//UE_LOGF(LogChaosJoint, VeryVerbose, "      Apply DR%d %f %f %f", 1, DR1.X, DR1.Y, DR1.Z);

		if (bRealTypeCompatibleWithISPC && bChaos_Joint_ISPC_Enabled)
		{
#if INTEL_ISPC
			ispc::ApplyRotationDelta2((ispc::FPBDJointSolver*)this, (ispc::FVector&)DR0, (ispc::FVector&)DR1);
#endif
		}
		else
		{
			if (IsDynamic(0))
			{
				Body(0).ApplyRotationDelta(DR0);
			}
			if (IsDynamic(1))
			{
				Body(1).ApplyRotationDelta(DR1);
			}

			UpdateDerivedState();
		}
	}

	void FPBDJointSolver::ApplyDelta(
		const int32 BodyIndex,
		const FVec3& DP,
		const FVec3& DR)
	{
		//UE_LOGF(LogChaosJoint, VeryVerbose, "      Apply DP%d %f %f %f", BodyIndex, DP.X, DP.Y, DP.Z);
		//UE_LOGF(LogChaosJoint, VeryVerbose, "      Apply DR%d %f %f %f", BodyIndex, DR.X, DR.Y, DR.Z);

		Body(BodyIndex).ApplyTransformDelta(DP, DR);

		UpdateDerivedState(BodyIndex);
	}


	void FPBDJointSolver::ApplyVelocityDelta(
		const int32 BodyIndex,
		const FVec3& DV,
		const FVec3& DW)
	{
		Body(BodyIndex).ApplyVelocityDelta(DV, DW);
	}


	void FPBDJointSolver::ApplyVelocityDelta(
		const FVec3& DV0,
		const FVec3& DW0,
		const FVec3& DV1,
		const FVec3& DW1)
	{
		Body(0).ApplyVelocityDelta(DV0, DW0);
		Body(1).ApplyVelocityDelta(DV1, DW1);
	}

	void FPBDJointSolver::ApplyAngularVelocityDelta(
		const FVec3& DW0,
		const FVec3& DW1)
	{
		Body(0).ApplyAngularVelocityDelta(DW0);
		Body(1).ApplyAngularVelocityDelta(DW1);
	}

	void FPBDJointSolver::ApplyPositionConstraint(
		const FReal JointStiffness,
		const FVec3& Axis,
		const FReal Delta,
		const FVec3& Connector0Correction,
		const int32 LinearHardLambdaIndex)
	{
		const FReal Stiffness = SolverStiffness * JointStiffness;
		// Project ConnectorXs[1] to the feasible space and apply impulse at the projected location. ConnectorXs[0] + Connector0Correction is the Projected location. The results are more stable this way.
		const FVec3 Arm0 = ConnectorXs[0] + Connector0Correction - P(0);
		const FVec3 Arm1 = ConnectorXs[1] - P(1);
		const FVec3 AngularAxis0 = FVec3::CrossProduct(Arm0, Axis);
		const FVec3 AngularAxis1 = FVec3::CrossProduct(Arm1, Axis);
		const FVec3 IA0 = Utilities::Multiply(InvI(0), AngularAxis0);
		const FVec3 IA1 = Utilities::Multiply(InvI(1), AngularAxis1);

		// Joint-space inverse mass
		const FReal II0 = FVec3::DotProduct(AngularAxis0, IA0);
		const FReal II1 = FVec3::DotProduct(AngularAxis1, IA1);
		const FReal IM = InvM(0) + II0 + InvM(1) + II1;

		// Apply constraint correction
		const FReal DLambda = Stiffness * Delta / IM;
		const FVec3 DX = Axis * DLambda;
		const FVec3 DP0 = InvM(0) * DX;
		const FVec3 DP1 = -InvM(1) * DX;
		const FVec3 DR0 = Utilities::Multiply(InvI(0), FVec3::CrossProduct(Arm0, DX));
		const FVec3 DR1 = Utilities::Multiply(InvI(1), FVec3::CrossProduct(Arm1, -DX));

		ApplyPositionDelta(DP0, DP1);
		ApplyRotationDelta(DR0, DR1);

		NetLinearImpulse += DX;

		if(LinearHardLambdaIndex >= 0)
		{
			LinearHardLambda[LinearHardLambdaIndex] += DLambda;
		}
	}


	void FPBDJointSolver::ApplyPositionConstraintSoft(
		const FReal Dt,
		const FReal JointStiffness,
		const FReal JointDamping,
		const bool bAccelerationMode,
		const FVec3& Axis,
		const FReal Delta,
		const FReal TargetVel,
		const FReal LambdaLimit,
		FReal& Lambda)
	{
		if (bRealTypeCompatibleWithISPC && bChaos_Joint_ISPC_Enabled)
		{
#if INTEL_ISPC
			FReal ReturnedLambda = Lambda;
			ispc::ApplyPositionConstraintSoft((ispc::FPBDJointSolver*)this, Dt, JointStiffness, JointDamping, bAccelerationMode, (ispc::FVector&)Axis, Delta, TargetVel, ReturnedLambda);
			Lambda = ReturnedLambda;
#endif
		}
		else
		{
			// Joint-space inverse mass
			const FVec3 AngularAxis0 = FVec3::CrossProduct(ConnectorXs[0] - P(0), Axis);
			const FVec3 AngularAxis1 = FVec3::CrossProduct(ConnectorXs[1] - P(1), Axis);
			const FVec3 IA0 = Utilities::Multiply(InvI(0), AngularAxis0);
			const FVec3 IA1 = Utilities::Multiply(InvI(1), AngularAxis1);
			const FReal II0 = FVec3::DotProduct(AngularAxis0, IA0);
			const FReal II1 = FVec3::DotProduct(AngularAxis1, IA1);
			const FReal II = (InvM(0) + II0 + InvM(1) + II1);
	
			FReal VelDt = 0;
			if (JointDamping > UE_KINDA_SMALL_NUMBER)
			{
				const FVec3 V0Dt = FVec3::CalculateVelocity(InitConnectorXs[0], ConnectorXs[0], 1.0f);
				const FVec3 V1Dt = FVec3::CalculateVelocity(InitConnectorXs[1], ConnectorXs[1], 1.0f);
				VelDt = TargetVel * Dt + FVec3::DotProduct(V0Dt - V1Dt, Axis);
			}
	
			const FReal SpringMassScale = (bAccelerationMode) ? 1.0f / (InvM(0) + InvM(1)) : 1.0f;
			const FReal S = SpringMassScale * JointStiffness * Dt * Dt;
			const FReal D = SpringMassScale * JointDamping * Dt;
			const FReal Multiplier = (FReal)1 / ((S + D) * II + (FReal)1);
			FReal DLambda = SolverStiffness * Multiplier * (S * Delta - D * VelDt - Lambda);
			if (LambdaLimit > SMALL_NUMBER)
			{
				DLambda = FMath::Clamp(DLambda, -LambdaLimit - Lambda, LambdaLimit - Lambda);
			}
	
			const FVec3 DP0 = (InvM(0) * DLambda) * Axis;
			const FVec3 DP1 = (-InvM(1) * DLambda) * Axis;
			const FVec3 DR0 = DLambda * Utilities::Multiply(InvI(0), AngularAxis0);
			const FVec3 DR1 = -DLambda * Utilities::Multiply(InvI(1), AngularAxis1);
	
			ApplyPositionDelta(DP0, DP1);
			ApplyRotationDelta(DR0, DR1);

			Lambda += DLambda;
			NetLinearImpulse += DLambda * Axis;
		}
	}
	

	void FPBDJointSolver::ApplyRotationConstraintKD(
		const int32 KIndex,
		const int32 DIndex,
		const FReal JointStiffness,
		const FVec3& Axis,
		const FReal Angle,
		const int32 AngularHardLambdaIndex)
	{
		const FReal Stiffness = SolverStiffness * JointStiffness;

		const FVec3 IA1 = Utilities::Multiply(InvI(DIndex), Axis);
		const FReal II1 = FVec3::DotProduct(Axis, IA1);
		const FReal DR = Stiffness * (Angle / II1);
		const FVec3 DR1 = IA1 * -DR;
		ApplyRotationDelta(DIndex, DR1);

		NetAngularImpulse += (KIndex == 0 )? DR * Axis : -DR * Axis;
		if (AngularHardLambdaIndex >= 0)
		{
			AngularHardLambda[AngularHardLambdaIndex] += DR;
		}
	}


	void FPBDJointSolver::ApplyRotationConstraintDD(
		const FReal JointStiffness,
		const FVec3& Axis,
		const FReal Angle,
		const int32 AngularHardLambdaIndex)
	{
		const FReal Stiffness = SolverStiffness * JointStiffness;

		// Joint-space inverse mass
		const FVec3 IA0 = Utilities::Multiply(InvI(0), Axis);
		const FVec3 IA1 = Utilities::Multiply(InvI(1), Axis);
		const FReal II0 = FVec3::DotProduct(Axis, IA0);
		const FReal II1 = FVec3::DotProduct(Axis, IA1);

		const FReal DR = Stiffness * Angle / (II0 + II1);
		const FVec3 DR0 = IA0 * DR;
		const FVec3 DR1 = IA1 * -DR;

		ApplyRotationDelta(DR0, DR1);

		NetAngularImpulse += Axis * DR;
		if (AngularHardLambdaIndex >= 0)
		{
			AngularHardLambda[AngularHardLambdaIndex] += DR;
		}
	}


	void FPBDJointSolver::ApplyRotationConstraint(
		const FReal JointStiffness,
		const FVec3& Axis,
		const FReal Angle,
		const int32 AngularHardLambdaIndex)
	{
		if (!IsDynamic(0))
		{
			ApplyRotationConstraintKD(0, 1, JointStiffness, Axis, Angle, AngularHardLambdaIndex);
		}
		else if (!IsDynamic(1))
		{
			ApplyRotationConstraintKD(1, 0, JointStiffness, Axis, -Angle, AngularHardLambdaIndex);
		}
		else
		{
			ApplyRotationConstraintDD(JointStiffness, Axis, Angle, AngularHardLambdaIndex);
		}
	}


	// See "XPBD: Position-Based Simulation of Compliant Constrained Dynamics"
	void FPBDJointSolver::ApplyRotationConstraintSoftKD(
		const int32 KIndex,
		const int32 DIndex,
		const FReal Dt,
		const FReal JointStiffness,
		const FReal JointDamping,
		const bool bAccelerationMode,
		const FVec3& Axis,
		const FReal Angle,
		const FReal AngVelTarget,
		const FReal LambdaLimit,
		FReal& Lambda)
	{
		check(!IsDynamic(KIndex));
		check(IsDynamic(DIndex));

		if (bRealTypeCompatibleWithISPC && bChaos_Joint_ISPC_Enabled)
		{
#if INTEL_ISPC
			FReal ReturnedLambda = Lambda;
			ispc::ApplyRotationConstraintSoftKD((ispc::FPBDJointSolver*)this, KIndex, DIndex, Dt, JointStiffness, JointDamping, bAccelerationMode, (ispc::FVector&) Axis, Angle, AngVelTarget, ReturnedLambda);
			Lambda = ReturnedLambda;
#endif
		}
		else
		{
			// World-space inverse mass
			const FVec3 IA1 = Utilities::Multiply(InvI(DIndex), Axis);

			// Joint-space inverse mass
			FReal II1 = FVec3::DotProduct(Axis, IA1);
			const FReal II = II1;

			// Damping angular velocity
			FReal AngVelDt = 0;
			if (JointDamping > UE_KINDA_SMALL_NUMBER)
			{
				const FVec3 W0Dt = FRotation3::CalculateAngularVelocity(InitConnectorRs[KIndex], ConnectorRs[KIndex], 1.0f);
				const FVec3 W1Dt = FRotation3::CalculateAngularVelocity(InitConnectorRs[DIndex], ConnectorRs[DIndex], 1.0f);
				AngVelDt = AngVelTarget * Dt + FVec3::DotProduct(Axis, W0Dt - W1Dt);
			}

			const FReal SpringMassScale = (bAccelerationMode) ? 1.0f / II : 1.0f;
			const FReal S = SpringMassScale * JointStiffness * Dt * Dt;
			const FReal D = SpringMassScale * JointDamping * Dt;
			const FReal Multiplier = (FReal)1 / ((S + D) * II + (FReal)1);
			FReal DLambda = SolverStiffness * Multiplier * (S * Angle - D * AngVelDt - Lambda);
			if (LambdaLimit > SMALL_NUMBER)
			{
				DLambda = FMath::Clamp(DLambda, -LambdaLimit - Lambda, LambdaLimit - Lambda);
			}

			//const FVec3 DR1 = IA1 * -DLambda;
			const FVec3 DR1 = Axis * -(DLambda * II1);

			ApplyRotationDelta(DIndex, DR1);
	
			Lambda += DLambda;
			NetAngularImpulse += (KIndex == 0 ? 1 : -1) * DLambda * Axis;
		}
	}

	// See "XPBD: Position-Based Simulation of Compliant Constrained Dynamics"
	void FPBDJointSolver::ApplyRotationConstraintSoftDD(
		const FReal Dt,
		const FReal JointStiffness,
		const FReal JointDamping,
		const bool bAccelerationMode,
		const FVec3& Axis,
		const FReal Angle,
		const FReal AngVelTarget,
		const FReal LambdaLimit,
		FReal& Lambda)
	{
		check(IsDynamic(0));
		check(IsDynamic(1));

		if (bRealTypeCompatibleWithISPC && bChaos_Joint_ISPC_Enabled)
		{
#if INTEL_ISPC
			FReal ReturnedLambda = Lambda;
			ispc::ApplyRotationConstraintSoftDD((ispc::FPBDJointSolver*)this, Dt, JointStiffness, JointDamping, bAccelerationMode, (ispc::FVector&) Axis, Angle, AngVelTarget, ReturnedLambda);
			Lambda = ReturnedLambda;
#endif
		}
		else
		{
			// World-space inverse mass
			const FVec3 IA0 = Utilities::Multiply(InvI(0), Axis);
			const FVec3 IA1 = Utilities::Multiply(InvI(1), Axis);

			// Joint-space inverse mass
			FReal II0 = FVec3::DotProduct(Axis, IA0);
			FReal II1 = FVec3::DotProduct(Axis, IA1);
			const FReal II = (II0 + II1);

			// Damping angular velocity
			FReal AngVelDt = 0;
			if (JointDamping > UE_KINDA_SMALL_NUMBER)
			{
				const FVec3 W0Dt = FRotation3::CalculateAngularVelocity(InitConnectorRs[0], ConnectorRs[0], 1.0f);
				const FVec3 W1Dt = FRotation3::CalculateAngularVelocity(InitConnectorRs[1], ConnectorRs[1], 1.0f);
				AngVelDt = AngVelTarget * Dt + FVec3::DotProduct(Axis, W0Dt - W1Dt);
			}

			const FReal SpringMassScale = (bAccelerationMode) ? 1.0f / II : 1.0f;
			const FReal S = SpringMassScale * JointStiffness * Dt * Dt;
			const FReal D = SpringMassScale * JointDamping * Dt;
			const FReal Multiplier = (FReal)1 / ((S + D) * II + (FReal)1);
			FReal DLambda = SolverStiffness * Multiplier * (S * Angle - D * AngVelDt - Lambda);
			if (LambdaLimit > SMALL_NUMBER)
			{
				DLambda = FMath::Clamp(DLambda, -LambdaLimit - Lambda, LambdaLimit - Lambda);
			}

			//const FVec3 DR0 = IA0 * DLambda;
			//const FVec3 DR1 = IA1 * -DLambda;
			const FVec3 DR0 = Axis * (DLambda * II0);
			const FVec3 DR1 = Axis * -(DLambda * II1);

			ApplyRotationDelta(DR0, DR1);

			Lambda += DLambda;
			NetAngularImpulse += DLambda * Axis;
		}
	}

	void FPBDJointSolver::ApplyRotationConstraintSoft(
		const FReal Dt,
		const FReal JointStiffness,
		const FReal JointDamping,
		const bool bAccelerationMode,
		const FVec3& Axis,
		const FReal Angle,
		const FReal AngVelTarget,
		const FReal LambdaLimit,
		FReal& Lambda)
	{
		if (!IsDynamic(0))
		{
			ApplyRotationConstraintSoftKD(0, 1, Dt, JointStiffness, JointDamping, bAccelerationMode, Axis, Angle, AngVelTarget, LambdaLimit, Lambda);
		}
		else if (!IsDynamic(1))
		{
			ApplyRotationConstraintSoftKD(1, 0, Dt, JointStiffness, JointDamping, bAccelerationMode, Axis, -Angle, -AngVelTarget, LambdaLimit, Lambda);
		}
		else
		{
			ApplyRotationConstraintSoftDD(Dt, JointStiffness, JointDamping, bAccelerationMode, Axis, Angle, AngVelTarget, LambdaLimit, Lambda);
		}
	}

	//
	//
	//////////////////////////////////////////////////////////////////////////
	//
	//

	void FPBDJointSolver::ApplyLockedRotationConstraints(
		const FReal Dt,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings,
		const bool bApplyTwist,
		const bool bApplySwing)
	{
		FVec3 Axis0, Axis1, Axis2;
		FPBDJointUtilities::GetLockedRotationAxes(ConnectorRs[0], ConnectorRs[1], Axis0, Axis1, Axis2);

		const FRotation3 R01 = ConnectorRs[0].Inverse() * ConnectorRs[1];

		if (bApplyTwist)
		{
			FReal TwistStiffness = FPBDJointUtilities::GetTwistStiffness(SolverSettings, JointSettings);
			ApplyRotationConstraint(TwistStiffness, Axis0, R01.X, (int32)EJointAngularConstraintIndex::Twist);

			// Update the angular violation
			//
			// TODO: Use post-correction error instead of pre-correction error?
			AngularViolation = FMath::Max(AngularViolation, FMath::Abs(R01.X));
		}

		if (bApplySwing)
		{
			FReal SwingStiffness = FPBDJointUtilities::GetSwingStiffness(SolverSettings, JointSettings);
			ApplyRotationConstraint(SwingStiffness, Axis1, R01.Y, (int32)EJointAngularConstraintIndex::Swing2);
			ApplyRotationConstraint(SwingStiffness, Axis2, R01.Z, (int32)EJointAngularConstraintIndex::Swing1);

			// Update the angular violation
			//
			// TODO: Use post-correction error instead of pre-correction error?
			AngularViolation = FMath::Max(AngularViolation, FMath::Abs(R01.Y));
			AngularViolation = FMath::Max(AngularViolation, FMath::Abs(R01.Z));
		}
	}

	void FPBDJointSolver::ApplyTwistConstraint(
		const FReal Dt,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings,
		const bool bUseSoftLimit)
	{
		FVec3 TwistAxis;
		FReal TwistAngle;
		FPBDJointUtilities::GetTwistAxisAngle(ConnectorRs[0], ConnectorRs[1], TwistAxis, TwistAngle);

		// Calculate the twist correction to apply to each body
		FReal DTwistAngle = 0;
		FReal TwistAngleMax = JointSettings.AngularLimits[(int32)EJointAngularConstraintIndex::Twist];
		if (TwistAngle > TwistAngleMax)
		{
			DTwistAngle = TwistAngle - TwistAngleMax;
		}
		else if (TwistAngle < -TwistAngleMax)
		{
			// Keep Twist error positive
			DTwistAngle = -TwistAngle - TwistAngleMax;
			TwistAxis = -TwistAxis;
		}

		UE_LOGF(LogChaosJoint, VeryVerbose, "    Twist Angle %f [Limit %f]", FMath::RadiansToDegrees(TwistAngle), FMath::RadiansToDegrees(TwistAngleMax));

		// Apply twist correction
		if (DTwistAngle > AngleTolerance)
		{
			if (bUseSoftLimit)
			{
				const FReal TwistStiffness = FPBDJointUtilities::GetSoftTwistStiffness(SolverSettings, JointSettings);
				const FReal TwistDamping = FPBDJointUtilities::GetSoftTwistDamping(SolverSettings, JointSettings);
				const bool bAccelerationMode = FPBDJointUtilities::GetAngularSoftAccelerationMode(SolverSettings, JointSettings);
				ApplyRotationConstraintSoft(Dt, TwistStiffness, TwistDamping, bAccelerationMode, TwistAxis, DTwistAngle, 0.0f, 0.0f, TwistSoftLambda);
			}
			else
			{
				FReal TwistStiffness = FPBDJointUtilities::GetTwistStiffness(SolverSettings, JointSettings);
				ApplyRotationConstraint(TwistStiffness, TwistAxis, DTwistAngle, (int32)EJointAngularConstraintIndex::Twist);
			}

			// Update the angular violation
			//
			// TODO: Use post-correction error instead of pre-correction error?
			AngularViolation = FMath::Max(AngularViolation, DTwistAngle);
		}
	}

	void FPBDJointSolver::ApplyConeConstraint(
		const FReal Dt,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings,
		const bool bUseSoftLimit)
	{
		FVec3 SwingAxisLocal;
		FReal DSwingAngle = 0.0f;

		const FReal Swing1Limit = JointSettings.AngularLimits[(int32)EJointAngularConstraintIndex::Swing1];
		const FReal Swing2Limit = JointSettings.AngularLimits[(int32)EJointAngularConstraintIndex::Swing2];
		FPBDJointUtilities::GetEllipticalConeAxisErrorLocal(ConnectorRs[0], ConnectorRs[1], Swing2Limit, Swing1Limit, SwingAxisLocal, DSwingAngle);

		UE_LOGF(LogChaosJoint, VeryVerbose, "    Cone Error %f [Limits %f %f]", FMath::RadiansToDegrees(DSwingAngle), FMath::RadiansToDegrees(Swing2Limit), FMath::RadiansToDegrees(Swing1Limit));

		const FVec3 SwingAxis = ConnectorRs[0] * SwingAxisLocal;

		// Apply swing correction to each body
		if (DSwingAngle > AngleTolerance)
		{
			if (bUseSoftLimit)
			{
				const FReal SoftSwingStiffness = FPBDJointUtilities::GetSoftSwingStiffness(SolverSettings, JointSettings);
				const FReal SoftSwingDamping = FPBDJointUtilities::GetSoftSwingDamping(SolverSettings, JointSettings);
				const bool bAccelerationMode = FPBDJointUtilities::GetAngularSoftAccelerationMode(SolverSettings, JointSettings);
				ApplyRotationConstraintSoft(Dt, SoftSwingStiffness, SoftSwingDamping, bAccelerationMode, SwingAxis, DSwingAngle, 0.0f, 0.0f, SwingSoftLambda);
			}
			else
			{
				FReal SwingStiffness = FPBDJointUtilities::GetSwingStiffness(SolverSettings, JointSettings);
				// For cone constraints, the lambda are all accumulated in Swing2
				ApplyRotationConstraint(SwingStiffness, SwingAxis, DSwingAngle, (int32)EJointAngularConstraintIndex::Swing2);
			}

			// Update the angular violation
			//
			// TODO: Use post-correction error instead of pre-correction error?
			AngularViolation = FMath::Max(AngularViolation, DSwingAngle);
		}
	}

	void FPBDJointSolver::ApplySingleLockedSwingConstraint(
		const FReal Dt,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings,
		const EJointAngularConstraintIndex SwingConstraintIndex,
		const bool bUseSoftLimit)
	{
		// NOTE: SwingAxis is not normalized in this mode. It has length Sin(SwingAngle).
		// Likewise, the SwingAngle is actually Sin(SwingAngle)
		FVec3 SwingAxis;
		FReal SwingAngle;
		FPBDJointUtilities::GetLockedSwingAxisAngle(ConnectorRs[0], ConnectorRs[1], SwingConstraintIndex, SwingAxis, SwingAngle);

		UE_LOGF(LogChaosJoint, VeryVerbose, "    LockedSwing%d Angle %f [Tolerance %f]", (SwingConstraintIndex == EJointAngularConstraintIndex::Swing1) ? 1 : 2, FMath::RadiansToDegrees(SwingAngle), FMath::RadiansToDegrees(AngleTolerance));

		// Apply swing correction
		FReal DSwingAngle = SwingAngle;
		if (FMath::Abs(DSwingAngle) > AngleTolerance)
		{
			if (bUseSoftLimit)
			{
				const FReal SoftSwingStiffness = FPBDJointUtilities::GetSoftSwingStiffness(SolverSettings, JointSettings);
				const FReal SoftSwingDamping = FPBDJointUtilities::GetSoftSwingDamping(SolverSettings, JointSettings);
				const bool bAccelerationMode = FPBDJointUtilities::GetAngularSoftAccelerationMode(SolverSettings, JointSettings);
				ApplyRotationConstraintSoft(Dt, SoftSwingStiffness, SoftSwingDamping, bAccelerationMode, SwingAxis, DSwingAngle, 0.0f, 0.0f, SwingSoftLambda);
			}
			else
			{
				const FReal SwingStiffness = FPBDJointUtilities::GetSwingStiffness(SolverSettings, JointSettings);
				ApplyRotationConstraint(SwingStiffness, SwingAxis, DSwingAngle, (int32)SwingConstraintIndex);
			}

			// Update the angular violation
			//
			// TODO: Use post-correction error instead of pre-correction error?
			AngularViolation = FMath::Max(AngularViolation, FMath::Abs(DSwingAngle));
		}
	}


	void FPBDJointSolver::ApplyDualConeSwingConstraint(
		const FReal Dt,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings,
		const EJointAngularConstraintIndex SwingConstraintIndex,
		const bool bUseSoftLimit)
	{
		FVec3 SwingAxis;
		FReal SwingAngle;
		FPBDJointUtilities::GetDualConeSwingAxisAngle(ConnectorRs[0], ConnectorRs[1], SwingConstraintIndex, SwingAxis, SwingAngle);

		// Calculate swing error we need to correct
		FReal DSwingAngle = 0;
		const FReal SwingAngleMax = JointSettings.AngularLimits[(int32)SwingConstraintIndex];
		if (SwingAngle > SwingAngleMax)
		{
			DSwingAngle = SwingAngle - SwingAngleMax;
		}
		else if (SwingAngle < -SwingAngleMax)
		{
			// Keep the error positive
			DSwingAngle = -SwingAngle - SwingAngleMax;
			SwingAxis = -SwingAxis;
		}

		UE_LOGF(LogChaosJoint, VeryVerbose, "    DualConeSwing%d Angle %f [Limit %f]", (SwingConstraintIndex == EJointAngularConstraintIndex::Swing1) ? 1 : 2, FMath::RadiansToDegrees(SwingAngle), FMath::RadiansToDegrees(SwingAngleMax));

		// Apply swing correction
		if (DSwingAngle > AngleTolerance)
		{
			if (bUseSoftLimit)
			{
				const FReal SoftSwingStiffness = FPBDJointUtilities::GetSoftSwingStiffness(SolverSettings, JointSettings);
				const FReal SoftSwingDamping = FPBDJointUtilities::GetSoftSwingDamping(SolverSettings, JointSettings);
				const bool bAccelerationMode = FPBDJointUtilities::GetAngularSoftAccelerationMode(SolverSettings, JointSettings);
				ApplyRotationConstraintSoft(Dt, SoftSwingStiffness, SoftSwingDamping, bAccelerationMode, SwingAxis, DSwingAngle, 0.0f, 0.0f, SwingSoftLambda);
			}
			else
			{
				const FReal SwingStiffness = FPBDJointUtilities::GetSwingStiffness(SolverSettings, JointSettings);
				ApplyRotationConstraint(SwingStiffness, SwingAxis, DSwingAngle, (int32)SwingConstraintIndex);
			}

			// Update the angular violation
			//
			// TODO: Use post-correction error instead of pre-correction error?
			AngularViolation = FMath::Max(AngularViolation, DSwingAngle);
		}
	}


	void FPBDJointSolver::ApplySwingConstraint(
		const FReal Dt,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings,
		const EJointAngularConstraintIndex SwingConstraintIndex,
		const bool bUseSoftLimit)
	{
		FVec3 SwingAxis;
		FReal SwingAngle;
		FPBDJointUtilities::GetSwingAxisAngle(ConnectorRs[0], ConnectorRs[1], SolverSettings.SwingTwistAngleTolerance, SwingConstraintIndex, SwingAxis, SwingAngle);

		// Calculate swing error we need to correct
		FReal DSwingAngle = 0;
		const FReal SwingAngleMax = JointSettings.AngularLimits[(int32)SwingConstraintIndex];
		if (SwingAngle > SwingAngleMax)
		{
			DSwingAngle = SwingAngle - SwingAngleMax;
		}
		else if (SwingAngle < -SwingAngleMax)
		{
			// Keep swing error positive
			DSwingAngle = -SwingAngle - SwingAngleMax;
			SwingAxis = -SwingAxis;
		}

		UE_LOGF(LogChaosJoint, VeryVerbose, "    Swing%d Angle %f [Limit %f]", (SwingConstraintIndex == EJointAngularConstraintIndex::Swing1) ? 1 : 2, FMath::RadiansToDegrees(SwingAngle), FMath::RadiansToDegrees(SwingAngleMax));

		// Apply swing correction
		if (DSwingAngle > AngleTolerance)
		{
			if (bUseSoftLimit)
			{
				const FReal SoftSwingStiffness = FPBDJointUtilities::GetSoftSwingStiffness(SolverSettings, JointSettings);
				const FReal SoftSwingDamping = FPBDJointUtilities::GetSoftSwingDamping(SolverSettings, JointSettings);
				const bool bAccelerationMode = FPBDJointUtilities::GetAngularSoftAccelerationMode(SolverSettings, JointSettings);
				ApplyRotationConstraintSoft(Dt, SoftSwingStiffness, SoftSwingDamping, bAccelerationMode, SwingAxis, DSwingAngle, 0.0f, 0.0f, SwingSoftLambda);
			}
			else
			{
				const FReal SwingStiffness = FPBDJointUtilities::GetSwingStiffness(SolverSettings, JointSettings);
				ApplyRotationConstraint(SwingStiffness, SwingAxis, DSwingAngle, (int32)SwingConstraintIndex);
			}

			// Update the angular violation
			//
			// TODO: Use post-correction error instead of pre-correction error?
			AngularViolation = FMath::Max(AngularViolation, DSwingAngle);
		}
	}


	void FPBDJointSolver::ApplySwingTwistDrives(
		const FReal Dt,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings,
		const bool bTwistDriveEnabled,
		const bool bSwing1DriveEnabled,
		const bool bSwing2DriveEnabled)
	{
		FRotation3 R1Target = ConnectorRs[0] * JointSettings.AngularDrivePositionTarget;
		R1Target.EnforceShortestArcWith(ConnectorRs[1]);
		FRotation3 R1Error = R1Target.Inverse() * ConnectorRs[1];
		FVec3 R1TwistAxisError = R1Error * FJointConstants::TwistAxis();

		// Angle approximation Angle ~= Sin(Angle) for small angles, underestimates for large angles
		const FReal DTwistAngle = 2.0f * R1Error.X;
		const FReal DSwing1Angle = R1TwistAxisError.Y;
		const FReal DSwing2Angle = -R1TwistAxisError.Z;

		const FReal AngularTwistDriveStiffness = FPBDJointUtilities::GetAngularTwistDriveStiffness(SolverSettings, JointSettings);
		const FReal AngularTwistDriveDamping = FPBDJointUtilities::GetAngularTwistDriveDamping(SolverSettings, JointSettings);
		const FReal AngularTwistDriveTorqueLimit = FPBDJointUtilities::GetAngularTwistDriveTorqueLimit(SolverSettings, JointSettings);
		const FReal AngularSwingDriveStiffness = FPBDJointUtilities::GetAngularSwingDriveStiffness(SolverSettings, JointSettings);
		const FReal AngularSwingDriveDamping = FPBDJointUtilities::GetAngularSwingDriveDamping(SolverSettings, JointSettings);
		const FReal AngularSwingDriveTorqueLimit = FPBDJointUtilities::GetAngularSwingDriveTorqueLimit(SolverSettings, JointSettings);
		const bool bAccelerationMode = FPBDJointUtilities::GetAngularDriveAccelerationMode(SolverSettings, JointSettings);
		const FReal AngularTwistDriveLambdaLimit = AngularTwistDriveTorqueLimit * Dt * Dt;
		const FReal AngularSwingDriveLambdaLimit = AngularSwingDriveTorqueLimit * Dt * Dt;

		const bool bUseTwistDrive = bTwistDriveEnabled && (((FMath::Abs(DTwistAngle) > AngleTolerance) && (AngularTwistDriveStiffness > 0.0f)) || (AngularTwistDriveDamping > 0.0f));
		if (bUseTwistDrive)
		{
			FReal ReturnedLambda = RotationDriveLambdas[(int32)EJointAngularConstraintIndex::Twist];
			const FVec3 TwistAxis = ConnectorRs[1] * FJointConstants::TwistAxis();
			ApplyRotationConstraintSoft(Dt, AngularTwistDriveStiffness, AngularTwistDriveDamping, bAccelerationMode, TwistAxis, DTwistAngle, JointSettings.AngularDriveVelocityTarget[(int32)EJointAngularConstraintIndex::Twist], AngularTwistDriveLambdaLimit, ReturnedLambda);
			RotationDriveLambdas[(int32)EJointAngularConstraintIndex::Twist] = ReturnedLambda;
		}

		const bool bUseSwing1Drive = bSwing1DriveEnabled && (((FMath::Abs(DSwing1Angle) > AngleTolerance) && (AngularSwingDriveStiffness > 0.0f)) || (AngularSwingDriveDamping > 0.0f));
		if (bUseSwing1Drive)
		{
			FReal ReturnedLambda = RotationDriveLambdas[(int32)EJointAngularConstraintIndex::Swing1];
			const FVec3 Swing1Axis = ConnectorRs[1] * FJointConstants::Swing1Axis();
			ApplyRotationConstraintSoft(Dt, AngularSwingDriveStiffness, AngularSwingDriveDamping, bAccelerationMode, Swing1Axis, DSwing1Angle, JointSettings.AngularDriveVelocityTarget[(int32)EJointAngularConstraintIndex::Swing1], AngularSwingDriveLambdaLimit, ReturnedLambda);
			RotationDriveLambdas[(int32)EJointAngularConstraintIndex::Swing1] = ReturnedLambda;
		}

		const bool bUseSwing2Drive = bSwing2DriveEnabled && (((FMath::Abs(DSwing2Angle) > AngleTolerance) && (AngularSwingDriveStiffness > 0.0f)) || (AngularSwingDriveDamping > 0.0f));
		if (bUseSwing2Drive)
		{
			FReal ReturnedLambda = RotationDriveLambdas[(int32)EJointAngularConstraintIndex::Swing2];
			const FVec3 Swing2Axis = ConnectorRs[1] * FJointConstants::Swing2Axis();
			ApplyRotationConstraintSoft(Dt, AngularSwingDriveStiffness, AngularSwingDriveDamping, bAccelerationMode, Swing2Axis, DSwing2Angle, JointSettings.AngularDriveVelocityTarget[(int32)EJointAngularConstraintIndex::Swing2], AngularSwingDriveLambdaLimit, ReturnedLambda);
			RotationDriveLambdas[(int32)EJointAngularConstraintIndex::Swing2] = ReturnedLambda;
		}
	}

	void FPBDJointSolver::ApplySLerpDrive(
		const FReal Dt,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		const FReal AngularDriveStiffness = FPBDJointUtilities::GetAngularSLerpDriveStiffness(SolverSettings, JointSettings);
		const FReal AngularDriveDamping = FPBDJointUtilities::GetAngularSLerpDriveDamping(SolverSettings, JointSettings);
		const bool bAccelerationMode = FPBDJointUtilities::GetAngularDriveAccelerationMode(SolverSettings, JointSettings);

		const FRotation3 R01 = ConnectorRs[0].Inverse() * ConnectorRs[1];
		FRotation3 TargetAngPos = JointSettings.AngularDrivePositionTarget;
		TargetAngPos.EnforceShortestArcWith(R01);
		const FRotation3 R1Error = TargetAngPos.Inverse() * R01;

		FReal AxisAngles[3] =
		{ 
			2.0f * Utilities::AsinEst(R1Error.X),
			2.0f * Utilities::AsinEst(R1Error.Y),
			2.0f * Utilities::AsinEst(R1Error.Z)
		};

		FVec3 Axes[3];
		ConnectorRs[1].ToMatrixAxes(Axes[0], Axes[1], Axes[2]);

		FReal AxisAngVel[3] = {0, 0, 0};
		if (!JointSettings.AngularDriveVelocityTarget.IsNearlyZero())
		{
			const FVec3 TargetAngVel = ConnectorRs[0] * JointSettings.AngularDriveVelocityTarget;
			AxisAngVel[0] = FVec3::DotProduct(TargetAngVel, Axes[0]);
			AxisAngVel[1] = FVec3::DotProduct(TargetAngVel, Axes[1]);
			AxisAngVel[2] = FVec3::DotProduct(TargetAngVel, Axes[2]);
		}

		ApplyRotationConstraintSoft(Dt, AngularDriveStiffness, AngularDriveDamping, bAccelerationMode, Axes[0], AxisAngles[0], AxisAngVel[0], 0, RotationDriveLambdas[0]);
		ApplyRotationConstraintSoft(Dt, AngularDriveStiffness, AngularDriveDamping, bAccelerationMode, Axes[1], AxisAngles[1], AxisAngVel[1], 0, RotationDriveLambdas[1]);
		ApplyRotationConstraintSoft(Dt, AngularDriveStiffness, AngularDriveDamping, bAccelerationMode, Axes[2], AxisAngles[2], AxisAngVel[2], 0, RotationDriveLambdas[2]);
	}


	// Kinematic-Dynamic bodies
	void FPBDJointSolver::ApplyPointPositionConstraintKD(
		const int32 KIndex,
		const int32 DIndex,
		const FReal Dt,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		check(!IsDynamic(KIndex));
		check(IsDynamic(DIndex));

		FReal Stiffness = SolverStiffness * FPBDJointUtilities::GetLinearStiffness(SolverSettings, JointSettings);
		const FVec3 CX = ConnectorXs[DIndex] - ConnectorXs[KIndex];

		UE_LOGF(LogChaosJoint, VeryVerbose, "    PointKD Delta %f [Limit %f]", CX.Size(), PositionTolerance);

		const FReal CXSquared = CX.SizeSquared();
		const FReal PositionToleranceSquared = PositionTolerance * PositionTolerance;
		if (CXSquared > PositionToleranceSquared)
		{
			if (bRealTypeCompatibleWithISPC && bChaos_Joint_ISPC_Enabled)
			{
#if INTEL_ISPC
				ispc::ApplyPointPositionConstraintKD((ispc::FPBDJointSolver*)this, KIndex, DIndex, (ispc::FVector&)CX, Stiffness);
#endif
			}
			else
			{
				// Calculate constraint correction
				FMatrix33 M = Utilities::ComputeJointFactorMatrix(ConnectorXs[DIndex] - P(DIndex), InvI(DIndex), InvM(DIndex));
				FMatrix33 MI = M.Inverse();
				const FVec3 DX = Stiffness * Utilities::Multiply(MI, CX);

				// Apply constraint correction
				const FVec3 DP1 = -InvM(DIndex) * DX;
				const FVec3 DR1 = Utilities::Multiply(InvI(DIndex), FVec3::CrossProduct(ConnectorXs[DIndex] - P(DIndex), -DX));

				ApplyDelta(DIndex, DP1, DR1);

				NetLinearImpulse += (KIndex == 0) ? DX : -DX;
			}

			// Update the linear violation
			//
			// TODO: Use post-correction violation
			LinearViolationSq = FMath::Max(LinearViolationSq, CXSquared);
		}
	}


	// Dynamic-Dynamic bodies
	void FPBDJointSolver::ApplyPointPositionConstraintDD(
		const FReal Dt,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		check(IsDynamic(0));
		check(IsDynamic(1));

		FReal Stiffness = SolverStiffness * FPBDJointUtilities::GetLinearStiffness(SolverSettings, JointSettings);
		const FVec3 CX = ConnectorXs[1] - ConnectorXs[0];

		UE_LOGF(LogChaosJoint, VeryVerbose, "    PointDD Delta %f [Limit %f]", CX.Size(), PositionTolerance);

		const FReal CXSquared = CX.SizeSquared();
		const FReal PositionToleranceSquared = PositionTolerance * PositionTolerance;
		if (CXSquared > PositionToleranceSquared)
		{
			if (bRealTypeCompatibleWithISPC && bChaos_Joint_ISPC_Enabled)
			{
#if INTEL_ISPC
				ispc::ApplyPointPositionConstraintDD((ispc::FPBDJointSolver*)this, (ispc::FVector&)CX, Stiffness);
#endif
			}
			else
			{
				// Calculate constraint correction
				FMatrix33 M0 = Utilities::ComputeJointFactorMatrix(ConnectorXs[0] - P(0), InvI(0), InvM(0));
				FMatrix33 M1 = Utilities::ComputeJointFactorMatrix(ConnectorXs[1] - P(1), InvI(1), InvM(1));
				FMatrix33 MI = (M0 + M1).Inverse();
				const FVec3 DX = Stiffness * Utilities::Multiply(MI, CX);

				// Apply constraint correction
				const FVec3 DP0 = InvM(0) * DX;
				const FVec3 DP1 = -InvM(1) * DX;
				const FVec3 DR0 = Utilities::Multiply(InvI(0), FVec3::CrossProduct(ConnectorXs[0] - P(0), DX));
				const FVec3 DR1 = Utilities::Multiply(InvI(1), FVec3::CrossProduct(ConnectorXs[1] - P(1), -DX));

				ApplyPositionDelta(DP0, DP1);
				ApplyRotationDelta(DR0, DR1);

				NetLinearImpulse += DX;
			}

			// Update the linear violation
			//
			// TODO: Use post-correction violation
			LinearViolationSq = FMath::Max(LinearViolationSq, CXSquared);
		}
	}


	void FPBDJointSolver::ApplySphericalPositionConstraint(
		const FReal Dt,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		FVec3 Axis;
		FReal Delta;
		FPBDJointUtilities::GetSphericalAxisDelta(ConnectorXs[0], ConnectorXs[1], Axis, Delta);

		const FReal Limit = JointSettings.LinearLimit;

		FReal Error = Delta - Limit;
		if (Error > PositionTolerance)
		{
			if (!FPBDJointUtilities::GetSoftLinearLimitEnabled(SolverSettings, JointSettings))
			{
				const FReal JointStiffness = FPBDJointUtilities::GetLinearStiffness(SolverSettings, JointSettings);
				const FVec3 Connector0Correction = Axis * Limit;
				ApplyPositionConstraint(JointStiffness, Axis, Error, Connector0Correction);
			}
			else
			{
				const FReal JointStiffness = FPBDJointUtilities::GetSoftLinearStiffness(SolverSettings, JointSettings);
				const FReal JointDamping = FPBDJointUtilities::GetSoftLinearDamping(SolverSettings, JointSettings);
				const bool bAccelerationMode = FPBDJointUtilities::GetLinearSoftAccelerationMode(SolverSettings, JointSettings);
				ApplyPositionConstraintSoft(Dt, JointStiffness, JointDamping, bAccelerationMode, Axis, Error, 0.0f, 0.0f, LinearSoftLambda);
			}

			// Update the linear violation
			//
			// TODO: Use post-correction violation
			LinearViolationSq = FMath::Max(LinearViolationSq, Error * Error);
		}
	}


	void FPBDJointSolver::ApplyCylindricalPositionConstraint(
		const FReal Dt,
		const int32 AxisIndex,
		const EJointMotionType AxialMotion,
		const EJointMotionType RadialMotion,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		check(AxialMotion != RadialMotion);

		FVec3 Axis, RadialAxis;
		FReal AxialDelta, RadialDelta;
		FPBDJointUtilities::GetCylindricalAxesDeltas(ConnectorRs[0], ConnectorXs[0], ConnectorXs[1], AxisIndex, Axis, AxialDelta, RadialAxis, RadialDelta);

		if (AxialDelta < 0.0f)
		{
			AxialDelta = -AxialDelta;
			Axis = -Axis;
		}
		
		const FReal AxialLimit = (AxialMotion == EJointMotionType::Locked) ? 0.0f : JointSettings.LinearLimit;
		FReal AxialError = AxialDelta - AxialLimit;

		if (AxialError > PositionTolerance)
		{
			if ((AxialMotion == EJointMotionType::Limited) && FPBDJointUtilities::GetSoftLinearLimitEnabled(SolverSettings, JointSettings))
			{
				// Soft Axial constraint
				const FReal JointStiffness = FPBDJointUtilities::GetSoftLinearStiffness(SolverSettings, JointSettings);
				const FReal JointDamping = FPBDJointUtilities::GetSoftLinearDamping(SolverSettings, JointSettings);
				const bool bAccelerationMode = FPBDJointUtilities::GetLinearSoftAccelerationMode(SolverSettings, JointSettings);
				ApplyPositionConstraintSoft(Dt, JointStiffness, JointDamping, bAccelerationMode, Axis, AxialError, 0.0f, 0.0f, LinearSoftLambda);
			}
			else if (AxialMotion != EJointMotionType::Free)
			{
				// Hard Axial constraint
	
				const FReal JointStiffness = FPBDJointUtilities::GetLinearStiffness(SolverSettings, JointSettings);
				const FVec3 Connector0Correction = ConnectorXs[1] - ConnectorXs[0] - AxialError * Axis;
				ApplyPositionConstraint(JointStiffness, Axis, AxialError, Connector0Correction, (int32)EJointCylindricalPositionConstraintType::Axial);
			}

			// Update the linear violation
			//
			// TODO: Use post-correction violation
			LinearViolationSq = FMath::Max(LinearViolationSq, AxialError * AxialError);
		}

		const FReal RadialLimit = (RadialMotion == EJointMotionType::Locked) ? 0.0f : JointSettings.LinearLimit;
		FReal RadialError = RadialDelta - RadialLimit;

		if (RadialError > PositionTolerance)
		{
			if ((RadialMotion == EJointMotionType::Limited) && FPBDJointUtilities::GetSoftLinearLimitEnabled(SolverSettings, JointSettings))
			{
				// Soft Radial constraint
				const FReal JointStiffness = FPBDJointUtilities::GetSoftLinearStiffness(SolverSettings, JointSettings);
				const FReal JointDamping = FPBDJointUtilities::GetSoftLinearDamping(SolverSettings, JointSettings);
				const bool bAccelerationMode = FPBDJointUtilities::GetLinearSoftAccelerationMode(SolverSettings, JointSettings);
				ApplyPositionConstraintSoft(Dt, JointStiffness, JointDamping, bAccelerationMode, RadialAxis, RadialError, 0.0f, 0.0f, LinearSoftLambda);
			}
			else if (RadialMotion != EJointMotionType::Free)
			{
				// Hard Radial constraint
				const FReal JointStiffness = FPBDJointUtilities::GetLinearStiffness(SolverSettings, JointSettings);
				const FVec3 Connector0Correction = ConnectorXs[1] - ConnectorXs[0] - RadialError * RadialAxis;
				ApplyPositionConstraint(JointStiffness, RadialAxis, RadialError, Connector0Correction, (int32)EJointCylindricalPositionConstraintType::Radial);
			}

			// Update the linear violation
			//
			// TODO: Use post-correction violation
			LinearViolationSq = FMath::Max(LinearViolationSq, RadialError * RadialError);
		}
	}


	void FPBDJointSolver::ApplyPlanarPositionConstraint(
		const FReal Dt,
		const int32 AxisIndex,
		const EJointMotionType AxialMotion,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		FVec3 Axis;
		FReal Delta;
		FPBDJointUtilities::GetPlanarAxisDelta(ConnectorRs[0], ConnectorXs[0], ConnectorXs[1], AxisIndex, Axis, Delta);

		if (Delta < 0.0f)
		{
			Delta = -Delta;
			Axis = -Axis;
		}

		const FReal Limit = (AxialMotion == EJointMotionType::Locked) ? 0 : JointSettings.LinearLimit;
		FReal Error = Delta - Limit;
		if (Error > PositionTolerance)
		{
			if ((AxialMotion == EJointMotionType::Limited) && FPBDJointUtilities::GetSoftLinearLimitEnabled(SolverSettings, JointSettings))
			{
				const FReal JointStiffness = FPBDJointUtilities::GetSoftLinearStiffness(SolverSettings, JointSettings);
				const FReal JointDamping = FPBDJointUtilities::GetSoftLinearDamping(SolverSettings, JointSettings);
				const bool bAccelerationMode = FPBDJointUtilities::GetLinearSoftAccelerationMode(SolverSettings, JointSettings);
				ApplyPositionConstraintSoft(Dt, JointStiffness, JointDamping, bAccelerationMode, Axis, Error, 0.0f, 0.0f, LinearSoftLambda);
			}
			else
			{
				const FReal JointStiffness = FPBDJointUtilities::GetLinearStiffness(SolverSettings, JointSettings);
				const FVec3 Connector0Correction = ConnectorXs[1] - ConnectorXs[0] - Axis * Error;
				ApplyPositionConstraint(JointStiffness, Axis, Error, Connector0Correction);
			}

			// Update the linear violation
			LinearViolationSq = FMath::Max(LinearViolationSq, Error * Error);
		}
	}


	void FPBDJointSolver::ApplyPositionDrive(
		const FReal Dt,
		const int32 AxisIndex,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings,
		const FVec3& Axis,
		const FReal DeltaPos,
		const FReal DeltaVel)
	{
		const FReal JointStiffness = FPBDJointUtilities::GetLinearDriveStiffness(SolverSettings, JointSettings, AxisIndex);
		const FReal JointDamping = FPBDJointUtilities::GetLinearDriveDamping(SolverSettings, JointSettings, AxisIndex);
		const bool bAccelerationMode = FPBDJointUtilities::GetLinearDriveAccelerationMode(SolverSettings, JointSettings);
		const FReal LambdaLimit = JointSettings.LinearDriveMaxForce[AxisIndex] * Dt * Dt;

		if ((FMath::Abs(DeltaPos) > PositionTolerance) || (JointDamping > 0.0f))
		{
			FReal ReturnedLambda = LinearDriveLambdas[AxisIndex];
			ApplyPositionConstraintSoft(Dt, JointStiffness, JointDamping, bAccelerationMode, Axis, DeltaPos, DeltaVel, LambdaLimit, ReturnedLambda);
			LinearDriveLambdas[AxisIndex] = ReturnedLambda;
		}
	}


	void FPBDJointSolver::ApplyPointProjection(
		const FReal Dt,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings,
		const FReal Alpha,
		FVec3& NetDP1,
		FVec3& NetDR1)
	{
		const FReal ProjectionPositionTolerance = 0.0f;//PositionTolerance;

		const FVec3 CX = ConnectorXs[1] - ConnectorXs[0];
		if (CX.Size() > ProjectionPositionTolerance)
		{
			FMatrix33 J = Utilities::ComputeJointFactorMatrix(ConnectorXs[1] - Body1().P(), InvI(1), InvM(1));
			const FMatrix33 IJ = J.Inverse();
			const FVec3 DX = Utilities::Multiply(IJ, CX);

			const FVec3 DP1 = -Alpha * InvM(1) * DX;
			const FVec3 DR1 = -Alpha * Utilities::Multiply(InvI(1), FVec3::CrossProduct(ConnectorXs[1] - Body1().P(), DX));
			ApplyDelta(1, DP1, DR1);

			NetDP1 += DP1;
			NetDR1 += DR1;
		}
	}

	void FPBDJointSolver::ApplySphereProjection(
		const FReal Dt,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings,
		const FReal Alpha,
		FVec3& NetDP1,
		FVec3& NetDR1)
	{
		const FReal ProjectionPositionTolerance = 0.0f;//PositionTolerance;

		FVec3 Axis;
		FReal Delta;
		FPBDJointUtilities::GetSphericalAxisDelta(ConnectorXs[0], ConnectorXs[1], Axis, Delta);
		const FReal Error = FMath::Max((FReal)0, Delta - JointSettings.LinearLimit);
		if (FMath::Abs(Error) > ProjectionPositionTolerance)
		{
			const FVec3 AngularAxis1 = FVec3::CrossProduct(ConnectorXs[1] - Body1().P(), Axis);
			const FVec3 IA1 = Utilities::Multiply(InvI(1), AngularAxis1);
			const FReal II1 = FVec3::DotProduct(AngularAxis1, IA1);
			const FReal IM = InvM(1) + II1;
			const FVec3 DX = Axis * Error / IM;

			const FVec3 DP1 = -Alpha * InvM(1) * DX;
			const FVec3 DR1 = -Alpha * Utilities::Multiply(InvI(1), FVec3::CrossProduct(ConnectorXs[1] - Body1().P(), DX));
			ApplyDelta(1, DP1, DR1);

			NetDP1 += DP1;
			NetDR1 += DR1;
		}
	}

	void FPBDJointSolver::ApplyTranslateProjection(
		const FReal Dt,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings,
		const FReal Alpha,
		FVec3& NetDP1,
		FVec3& NetDR1)
	{
		const FReal ProjectionPositionTolerance = 0.0f;//PositionTolerance;

		FVec3 Axis;
		FReal Delta;
		FPBDJointUtilities::GetSphericalAxisDelta(ConnectorXs[0], ConnectorXs[1], Axis, Delta);
		const FReal Error = FMath::Max((FReal)0, Delta - JointSettings.LinearLimit);
		if (Error > ProjectionPositionTolerance)
		{
			const FVec3 DP1 = -Alpha * Error * Axis;
			ApplyPositionDelta(1, DP1);
			
			NetDP1 += DP1;
		}
	}

	void FPBDJointSolver::ApplyConeProjection(
		const FReal Dt,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings,
		const FReal Alpha,
		const bool bPositionLocked,
		FVec3& NetDP1,
		FVec3& NetDR1)
	{
		const FReal ProjectionAngleTolerance = 0.0f;//AngleTolerance;

		FVec3 SwingAxisLocal;
		FReal DSwingAngle = 0.0f;
		const FReal Swing1Limit = JointSettings.AngularLimits[(int32)EJointAngularConstraintIndex::Swing1];
		const FReal Swing2Limit = JointSettings.AngularLimits[(int32)EJointAngularConstraintIndex::Swing2];
		FPBDJointUtilities::GetEllipticalConeAxisErrorLocal(ConnectorRs[0], ConnectorRs[1], Swing2Limit, Swing1Limit, SwingAxisLocal, DSwingAngle);
		FVec3 SwingAxis = ConnectorRs[0] * SwingAxisLocal;
		if (DSwingAngle > ProjectionAngleTolerance)
		{
			const FVec3 DR1 = -Alpha * DSwingAngle * SwingAxis;
			FVec3 DP1 = FVec3(0);
			if (bPositionLocked)
			{
				DP1 = -Alpha * FVec3::CrossProduct(DR1, ConnectorXs[1] - Body1().P());
			}
			ApplyDelta(1, DP1, DR1);

			NetDP1 += DP1;
			NetDR1 += DR1;
		}
	}

	void FPBDJointSolver::ApplySwingProjection(
		const FReal Dt,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings,
		const EJointAngularConstraintIndex SwingConstraintIndex,
		const FReal Alpha,
		const bool bPositionLocked,
		FVec3& NetDP1,
		FVec3& NetDR1)
	{
		const FReal ProjectionAngleTolerance = 0.0f;//AngleTolerance;

		FVec3 SwingAxis;
		FReal SwingAngle;
		FPBDJointUtilities::GetSwingAxisAngle(ConnectorRs[0], ConnectorRs[1], SolverSettings.SwingTwistAngleTolerance, SwingConstraintIndex, SwingAxis, SwingAngle);

		// Calculate swing error we need to correct
		FReal DSwingAngle = 0;
		const FReal SwingAngleMax = JointSettings.AngularLimits[(int32)SwingConstraintIndex];
		if (SwingAngle > SwingAngleMax)
		{
			DSwingAngle = SwingAngle - SwingAngleMax;
		}
		else if (SwingAngle < -SwingAngleMax)
		{
			DSwingAngle = SwingAngle + SwingAngleMax;
		}

		UE_LOGF(LogChaosJoint, VeryVerbose, "    Swing%d Angle %f [Limit %f]", (SwingConstraintIndex == EJointAngularConstraintIndex::Swing1) ? 1 : 2, FMath::RadiansToDegrees(SwingAngle), FMath::RadiansToDegrees(SwingAngleMax));

		// Apply swing correction
		if (FMath::Abs(DSwingAngle) > ProjectionAngleTolerance)
		{
			const FVec3 DR1 = -Alpha * DSwingAngle * SwingAxis;
			FVec3 DP1 = FVec3(0);
			if (bPositionLocked)
			{
				DP1 = -Alpha * FVec3::CrossProduct(DR1, ConnectorXs[1] - Body1().P());
			}
			ApplyDelta(1, DP1, DR1);

			NetDP1 += DP1;
			NetDR1 += DR1;
		}
	}

	void FPBDJointSolver::ApplySingleLockedSwingProjection(
		const FReal Dt,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings,
		const EJointAngularConstraintIndex SwingConstraintIndex,
		const FReal Alpha,
		const bool bPositionLocked,
		FVec3& NetDP1,
		FVec3& NetDR1)
	{
		ApplySwingProjection(Dt, SolverSettings, JointSettings, SwingConstraintIndex, Alpha, bPositionLocked, NetDP1, NetDR1);
	}

	void FPBDJointSolver::ApplyDoubleLockedSwingProjection(
		const FReal Dt,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings,
		const FReal Alpha,
		const bool bPositionLocked,
		FVec3& NetDP1,
		FVec3& NetDR1)
	{
		const FReal ProjectionAngleTolerance = 0.0f;//AngleTolerance;

		FVec3 SwingAxisLocal;
		FReal DSwingAngle = 0.0f;
		FPBDJointUtilities::GetCircularConeAxisErrorLocal(ConnectorRs[0], ConnectorRs[1], 0.0f, SwingAxisLocal, DSwingAngle);
		FVec3 SwingAxis = ConnectorRs[0] * SwingAxisLocal;
		if (DSwingAngle > ProjectionAngleTolerance)
		{
			const FVec3 DR1 = -Alpha * DSwingAngle * SwingAxis;
			FVec3 DP1 = FVec3(0);
			if (bPositionLocked)
			{
				DP1 = -Alpha * FVec3::CrossProduct(DR1, ConnectorXs[1] - Body1().P());
			}
			ApplyDelta(1, DP1, DR1);

			NetDP1 += DP1;
			NetDR1 += DR1;
		}
	}

	void FPBDJointSolver::ApplyDualConeSwingProjection(
		const FReal Dt,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings,
		const EJointAngularConstraintIndex SwingConstraintIndex,
		const FReal Alpha,
		const bool bPositionLocked,
		FVec3& NetDP1,
		FVec3& NetDR1)
	{
		const FReal ProjectionAngleTolerance = 0.0f;//AngleTolerance;

		FVec3 SwingAxis;
		FReal SwingAngle;
		FPBDJointUtilities::GetDualConeSwingAxisAngle(ConnectorRs[0], ConnectorRs[1], SwingConstraintIndex, SwingAxis, SwingAngle);

		// Calculate swing error we need to correct
		FReal DSwingAngle = 0;
		const FReal SwingAngleMax = JointSettings.AngularLimits[(int32)SwingConstraintIndex];
		if (SwingAngle > SwingAngleMax)
		{
			DSwingAngle = SwingAngle - SwingAngleMax;
		}
		else if (SwingAngle < -SwingAngleMax)
		{
			DSwingAngle = SwingAngle + SwingAngleMax;
		}

		// Apply swing correction
		if (FMath::Abs(DSwingAngle) > ProjectionAngleTolerance)
		{
			const FVec3 DR1 = -Alpha * DSwingAngle * SwingAxis;
			FVec3 DP1 = FVec3(0);
			if (bPositionLocked)
			{
				DP1 = -Alpha * FVec3::CrossProduct(DR1, ConnectorXs[1] - Body1().P());
			}
			ApplyDelta(1, DP1, DR1);

			NetDP1 += DP1;
			NetDR1 += DR1;
		}
	}

	void FPBDJointSolver::ApplyTwistProjection(
		const FReal Dt,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings,
		const FReal Alpha,
		const bool bPositionLocked,
		FVec3& NetDP1,
		FVec3& NetDR1)
	{
		const FReal ProjectionAngleTolerance = 0.0f;//AngleTolerance;

		FVec3 TwistAxis;
		FReal TwistAngle;
		FPBDJointUtilities::GetTwistAxisAngle(ConnectorRs[0], ConnectorRs[1], TwistAxis, TwistAngle);
		FReal DTwistAngle = 0;
		const FReal TwistLimit = JointSettings.AngularLimits[(int32)EJointAngularConstraintIndex::Twist];
		if (TwistAngle > TwistLimit)
		{
			DTwistAngle = TwistAngle - TwistLimit;
		}
		else if (TwistAngle < -TwistLimit)
		{
			DTwistAngle = TwistAngle + TwistLimit;
		}

		if (FMath::Abs(DTwistAngle) > ProjectionAngleTolerance)
		{
			const FVec3 DR1 = -Alpha * DTwistAngle * TwistAxis;
			FVec3 DP1 = FVec3(0);
			if (bPositionLocked)
			{
				DP1 = -Alpha * FVec3::CrossProduct(DR1, ConnectorXs[1] - Body1().P());
			}
			ApplyDelta(1, DP1, DR1);

			NetDP1 += DP1;
			NetDR1 += DR1;
		}
	}

	void FPBDJointSolver::ApplyVelocityProjection(
		const FReal Dt,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings,
		const FReal Alpha,
		const FVec3& DP1,
		const FVec3& DR1)
	{
		const FVec3 DV1 = Alpha * DP1 / Dt;
		const FVec3 DW1 = Alpha * DR1 / Dt;
		ApplyVelocityDelta(1, DV1, DW1);
	}

	void FPBDJointSolver::ApplyLinearVelocityConstraint(
		const FReal Stiffness,
		const FVec3& Axis,
		const FVec3& Connector0Correction,
		const FReal TargetVel)
	{
		const FVec3 Arm0 = ConnectorXs[0] + Connector0Correction - Body0().P();
		const FVec3 Arm1 = ConnectorXs[1] - Body1().P();
		const FVec3 CV0 = V(0) + FVec3::CrossProduct(W(0), Arm0);
		const FVec3 CV1 = V(1) + FVec3::CrossProduct(W(1), Arm1);
		const FVec3 CV = CV1 - CV0;

		const FVec3 AngularAxis0 = FVec3::CrossProduct(Arm0, Axis);
		const FVec3 AngularAxis1 = FVec3::CrossProduct(Arm1, Axis);
		const FVec3 IA0 = Utilities::Multiply(InvI(0), AngularAxis0);
		const FVec3 IA1 = Utilities::Multiply(InvI(1), AngularAxis1);
		const FReal II0 = FVec3::DotProduct(AngularAxis0, IA0);
		const FReal II1 = FVec3::DotProduct(AngularAxis1, IA1);
		const FReal II = (InvM(0) + II0 + InvM(1) + II1);

		const FVec3 Impulse = Stiffness * (FVec3::DotProduct(CV, Axis) - TargetVel) / II * Axis;

		const FVec3 DV0 = InvM(0) * Impulse;
		const FVec3 DV1 = -InvM(1) * Impulse;
		const FVec3 DW0 = Utilities::Multiply(InvI(0), FVec3::CrossProduct(Arm0, Impulse));
		const FVec3 DW1 = Utilities::Multiply(InvI(1), FVec3::CrossProduct(Arm1, -Impulse));

		ApplyVelocityDelta(DV0, DW0, DV1, DW1);
	}

	void FPBDJointSolver::ApplyPointVelocityConstraint(
		const FReal Dt,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		if (!NetLinearImpulse.IsNearlyZero() && !FPBDJointUtilities::GetSoftLinearLimitEnabled(SolverSettings, JointSettings))
		{
			FReal Stiffness = SolverStiffness * FPBDJointUtilities::GetLinearStiffness(SolverSettings, JointSettings);
			const FVec3 CV0 = V(0) + FVec3::CrossProduct(W(0), ConnectorXs[0] - Body0().P());
			const FVec3 CV1 = V(1) + FVec3::CrossProduct(W(1), ConnectorXs[1] - Body1().P());
			const FVec3 CV = CV1 - CV0;

			UE_LOGF(LogChaosJoint, VeryVerbose, "    PointVel Delta %f", CV.Size());
			// Calculate constraint correction
			FMatrix33 M0 = FMatrix33::Zero;
			FMatrix33 M1 = FMatrix33::Zero;
			if (InvM(0) > UE_SMALL_NUMBER)
			{
				M0 = Utilities::ComputeJointFactorMatrix(ConnectorXs[0] - Body0().P(), InvI(0), InvM(0));
			}
			if (InvM(1) > UE_SMALL_NUMBER)
			{
				M1 = Utilities::ComputeJointFactorMatrix(ConnectorXs[1] - Body1().P(), InvI(1), InvM(1));
			}
			FMatrix33 MI = (M0 + M1).Inverse();
			const FVec3 Impulse = Stiffness * Utilities::Multiply(MI, CV);

			// Apply constraint correction
			const FVec3 DV0 = InvM(0) * Impulse;
			const FVec3 DV1 = -InvM(1) * Impulse;
			const FVec3 DW0 = Utilities::Multiply(InvI(0), FVec3::CrossProduct(ConnectorXs[0] - Body0().P(), Impulse));
			const FVec3 DW1 = Utilities::Multiply(InvI(1), FVec3::CrossProduct(ConnectorXs[1] - Body1().P(), -Impulse));

			ApplyVelocityDelta(DV0, DW0, DV1, DW1);
		}
	}

	void FPBDJointSolver::ApplySphericalVelocityConstraint(
		const FReal Dt,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		if(!NetLinearImpulse.IsNearlyZero() && !FPBDJointUtilities::GetSoftLinearLimitEnabled(SolverSettings, JointSettings))
		{
			FVec3 Axis;
			FReal Delta;
			FPBDJointUtilities::GetSphericalAxisDelta(ConnectorXs[0], ConnectorXs[1], Axis, Delta);

			const FReal Stiffness = SolverStiffness * FPBDJointUtilities::GetLinearStiffness(SolverSettings, JointSettings);
			const FVec3 DConnectorX = ConnectorXs[1] - ConnectorXs[0];
			const FVec3 Connector0Correction = Delta > JointSettings.LinearLimit ? Axis * JointSettings.LinearLimit : DConnectorX;
			FReal TargetVel = 0.0f;
			if (JointSettings.LinearRestitution != 0.0f)
			{
				const FReal InitVel = InitConstraintAxisLinearVelocities[0];
				TargetVel = InitVel > Chaos_Joint_LinearVelocityThresholdToApplyRestitution ? -JointSettings.LinearRestitution * InitVel : 0.0f; 
			}
			ApplyLinearVelocityConstraint(Stiffness, Axis, Connector0Correction, TargetVel);
		}
	}

	void FPBDJointSolver::ApplyCylindricalVelocityConstraint(
		const FReal Dt,
		const int32 AxisIndex,
		const EJointMotionType AxialMotion,
		const EJointMotionType RadialMotion,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		check(AxialMotion != RadialMotion);

		if(!NetLinearImpulse.IsNearlyZero() && !FPBDJointUtilities::GetSoftLinearLimitEnabled(SolverSettings, JointSettings))
		{
			FVec3 Axis, RadialAxis;
			FReal AxialDelta, RadialDelta;
			FPBDJointUtilities::GetCylindricalAxesDeltas(ConnectorRs[0], ConnectorXs[0], ConnectorXs[1], AxisIndex, Axis, AxialDelta, RadialAxis, RadialDelta);
			const FVec3 DConnectorX = ConnectorXs[1] - ConnectorXs[0];

			if (LinearHardLambda[(int32)EJointCylindricalPositionConstraintType::Axial] > UE_SMALL_NUMBER)
			{
				const FReal Stiffness = SolverStiffness * FPBDJointUtilities::GetLinearStiffness(SolverSettings, JointSettings);

				// This flipping of axis might result in undesired restitution velocity if the current axis and the axis at the beginning of the frame are pointing at very different directions.
				// On the other hand, Quasipbd restitution in general works only on the assumption that constraint axes do not change much within one frame. 
				if (AxialDelta < 0.0f)
				{
					AxialDelta = -AxialDelta;
					Axis = -Axis;
				}
				
				const FReal AxialLimit = (AxialMotion == EJointMotionType::Locked) ? 0.0f : JointSettings.LinearLimit;
				const FReal AxialError = AxialDelta - AxialLimit;

				const FVec3 Connector0Correction = DConnectorX - AxialError * Axis;
				FReal TargetVel = 0.0f;
				if (AxialMotion == EJointMotionType::Limited && JointSettings.LinearRestitution != 0.0f)
				{
					const FReal InitVel = InitConstraintAxisLinearVelocities[(int32)EJointCylindricalPositionConstraintType::Axial];
					TargetVel = InitVel > Chaos_Joint_LinearVelocityThresholdToApplyRestitution ? -JointSettings.LinearRestitution * InitVel : 0.0f; 
				}

				ApplyLinearVelocityConstraint(Stiffness, Axis, Connector0Correction, TargetVel);
			}

			if (LinearHardLambda[(int32)EJointCylindricalPositionConstraintType::Radial] > UE_SMALL_NUMBER)
			{
				const FReal Stiffness = SolverStiffness * FPBDJointUtilities::GetLinearStiffness(SolverSettings, JointSettings);

				const FReal RadialLimit = (RadialMotion == EJointMotionType::Locked) ? 0.0f : JointSettings.LinearLimit;
				const FReal RadialError = FMath::Max(RadialDelta - RadialLimit, 0.0f);
				const FVec3 Connector0Correction = DConnectorX - RadialError * RadialAxis;
				FReal TargetVel = 0.0f;
				if (RadialMotion == EJointMotionType::Limited && JointSettings.LinearRestitution != 0.0f)
				{
					const FReal InitVel = InitConstraintAxisLinearVelocities[(int32)EJointCylindricalPositionConstraintType::Radial];
					TargetVel = InitVel > Chaos_Joint_LinearVelocityThresholdToApplyRestitution ? -JointSettings.LinearRestitution * InitVel : 0.0f; 
				}
				ApplyLinearVelocityConstraint(Stiffness, RadialAxis, Connector0Correction, TargetVel);
			}
		}
	}

	void FPBDJointSolver::ApplyPlanarVelocityConstraint(
		const FReal Dt,
		const int32 AxisIndex,
		const EJointMotionType AxialMotion,
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		if(!NetLinearImpulse.IsNearlyZero() && !FPBDJointUtilities::GetSoftLinearLimitEnabled(SolverSettings, JointSettings))
		{
			FVec3 Axis;
			FReal Delta;
			FPBDJointUtilities::GetPlanarAxisDelta(ConnectorRs[0], ConnectorXs[0], ConnectorXs[1], AxisIndex, Axis, Delta);

			const FReal Stiffness = SolverStiffness * FPBDJointUtilities::GetLinearStiffness(SolverSettings, JointSettings);

			if (Delta < 0.0f)
			{
				Delta = -Delta;
				Axis = -Axis;
			}
			const FReal Limit = (AxialMotion == EJointMotionType::Locked) ? 0 : JointSettings.LinearLimit;
			const FReal Error = Delta - Limit;

			const FVec3 DConnectorX = ConnectorXs[1] - ConnectorXs[0];
			const FVec3 Connector0Correction = DConnectorX - Axis * Error;

			FReal TargetVel = 0.0f;
			if (AxialMotion == EJointMotionType::Limited && JointSettings.LinearRestitution != 0.0f)
			{
				const FReal InitVel = InitConstraintAxisLinearVelocities[0];
				TargetVel = InitVel > Chaos_Joint_LinearVelocityThresholdToApplyRestitution ? -JointSettings.LinearRestitution * InitVel : 0.0f; 
			}
			ApplyLinearVelocityConstraint(Stiffness, Axis, Connector0Correction, TargetVel);
		}
	}

	void FPBDJointSolver::CalculateConstraintAxisLinearVelocities(
		const FPBDJointSettings& JointSettings,
		FVec3& ConstraintAxisLinearVelocities) const
	{
		bool bHasPositionConstraints =
			(JointSettings.LinearMotionTypes[0] != EJointMotionType::Free)
			|| (JointSettings.LinearMotionTypes[1] != EJointMotionType::Free)
			|| (JointSettings.LinearMotionTypes[2] != EJointMotionType::Free);
		if (!bHasPositionConstraints)
		{
			return;
		}

		const TVector<EJointMotionType, 3>& LinearMotion = JointSettings.LinearMotionTypes;
		const TVector<bool, 3> bLinearLocked =
		{
			(LinearMotion[0] == EJointMotionType::Locked),
			(LinearMotion[1] == EJointMotionType::Locked),
			(LinearMotion[2] == EJointMotionType::Locked),
		};
		const TVector<bool, 3> bLinearLimited =
		{
			(LinearMotion[0] == EJointMotionType::Limited),
			(LinearMotion[1] == EJointMotionType::Limited),
			(LinearMotion[2] == EJointMotionType::Limited),
		};

		if (bLinearLocked[0] && bLinearLocked[1] && bLinearLocked[2])
		{
			
		}
		else if (bLinearLimited[0] && bLinearLimited[1] && bLinearLimited[2])
		{
			CalculateSphericalConstraintAxisLinearVelocities(JointSettings, ConstraintAxisLinearVelocities);
		}
		else if (bLinearLocked[1] && bLinearLocked[2] && !bLinearLocked[0])
		{
			// Line constraint along X axis
			CalculateCylindricalConstraintAxisLinearVelocities(0, LinearMotion[0], EJointMotionType::Locked, JointSettings, ConstraintAxisLinearVelocities);
		}
		else if (bLinearLocked[0] && bLinearLocked[2] && !bLinearLocked[1])
		{
			// Line constraint along Y axis
			CalculateCylindricalConstraintAxisLinearVelocities(1, LinearMotion[1], EJointMotionType::Locked, JointSettings, ConstraintAxisLinearVelocities);
		}
		else if (bLinearLocked[0] && bLinearLocked[1] && !bLinearLocked[2])
		{
			// Line constraint along Z axis
			CalculateCylindricalConstraintAxisLinearVelocities(2, LinearMotion[2], EJointMotionType::Locked, JointSettings, ConstraintAxisLinearVelocities);
		}
		else if (bLinearLimited[1] && bLinearLimited[2] && !bLinearLimited[0])
		{
			// Cylindrical constraint along X axis
			CalculateCylindricalConstraintAxisLinearVelocities(0, LinearMotion[0], EJointMotionType::Limited, JointSettings, ConstraintAxisLinearVelocities);
		}
		else if (bLinearLimited[0] && bLinearLimited[2] && !bLinearLimited[1])
		{
			// Cylindrical constraint along Y axis
			CalculateCylindricalConstraintAxisLinearVelocities(1, LinearMotion[1], EJointMotionType::Limited, JointSettings, ConstraintAxisLinearVelocities);
		}
		else if (bLinearLimited[0] && bLinearLimited[1] && !bLinearLimited[2])
		{
			// Cylindrical constraint along Z axis
			CalculateCylindricalConstraintAxisLinearVelocities(2, LinearMotion[2], EJointMotionType::Limited, JointSettings, ConstraintAxisLinearVelocities);
		}
		else if (bLinearLimited[0])
		{
			// Planar constraint along X axis
			CalculatePlanarConstraintAxisLinearVelocities(0, LinearMotion[0], JointSettings, ConstraintAxisLinearVelocities);
		}
		else if (bLinearLimited[1])
		{
			// Planar constraint along Y axis
			CalculatePlanarConstraintAxisLinearVelocities(1, LinearMotion[1], JointSettings, ConstraintAxisLinearVelocities);
		}
		else if (bLinearLimited[2])
		{
			// Planar constraint along Z axis
			CalculatePlanarConstraintAxisLinearVelocities(2, LinearMotion[2], JointSettings, ConstraintAxisLinearVelocities);
		}
	}

	void FPBDJointSolver::CalculateSphericalConstraintAxisLinearVelocities(
			const FPBDJointSettings& JointSettings,
			FVec3& ConstraintAxisLinearVelocities) const
	{
		FVec3 Axis;
		FReal Delta;
		FPBDJointUtilities::GetSphericalAxisDelta(ConnectorXs[0], ConnectorXs[1], Axis, Delta);
		const FVec3 DConnectorX = ConnectorXs[1] - ConnectorXs[0];
		const FVec3 Connector0Correction = Delta > JointSettings.LinearLimit ? Axis * JointSettings.LinearLimit : DConnectorX;

		const FVec3 Arm0 = ConnectorXs[0] + Connector0Correction - Body0().P();
		const FVec3 Arm1 = ConnectorXs[1] - Body1().P();
		const FVec3 CV0 = V(0) + FVec3::CrossProduct(W(0), Arm0);
		const FVec3 CV1 = V(1) + FVec3::CrossProduct(W(1), Arm1);
		const FVec3 CV = CV1 - CV0;

		ConstraintAxisLinearVelocities[0] = FVec3::DotProduct(CV, Axis);
	}

	void FPBDJointSolver::CalculateCylindricalConstraintAxisLinearVelocities(
			const int32 AxisIndex,
			const EJointMotionType AxialMotion,
			const EJointMotionType RadialMotion,
			const FPBDJointSettings& JointSettings,
			FVec3& ConstraintAxisLinearVelocities) const
	{
		FVec3 Axis, RadialAxis;
		FReal AxialDelta, RadialDelta;
		FPBDJointUtilities::GetCylindricalAxesDeltas(ConnectorRs[0], ConnectorXs[0], ConnectorXs[1], AxisIndex, Axis, AxialDelta, RadialAxis, RadialDelta);
		const FVec3 DConnectorX = ConnectorXs[1] - ConnectorXs[0];

		if (AxialMotion == EJointMotionType::Limited)
		{
			if (AxialDelta < 0.0f)
			{
				AxialDelta = -AxialDelta;
				Axis = -Axis;
			}
			const FReal AxialLimit = (AxialMotion == EJointMotionType::Locked) ? 0.0f :JointSettings.LinearLimit;
			const FReal AxialError = AxialDelta - AxialLimit;
			
			const FVec3 Connector0Correction = DConnectorX - AxialError * Axis;

			const FVec3 Arm0 = ConnectorXs[0] + Connector0Correction - P(0);
			const FVec3 Arm1 = ConnectorXs[1] - P(1);
			const FVec3 CV0 = V(0) + FVec3::CrossProduct(W(0), Arm0);
			const FVec3 CV1 = V(1) + FVec3::CrossProduct(W(1), Arm1);
			const FVec3 CV = CV1 - CV0;

			ConstraintAxisLinearVelocities[(int32)EJointCylindricalPositionConstraintType::Axial] = FVec3::DotProduct(CV, Axis);
		}

		if (RadialMotion == EJointMotionType::Limited)
		{
			const FReal RadialLimit = (RadialMotion == EJointMotionType::Locked) ? 0.0f : JointSettings.LinearLimit;
			const FReal RadialError = FMath::Max(RadialDelta - RadialLimit, 0.0f);
			const FVec3 Connector0Correction = DConnectorX - RadialError * RadialAxis;
			
			const FVec3 Arm0 = ConnectorXs[0] + Connector0Correction - P(0);
			const FVec3 Arm1 = ConnectorXs[1] - P(1);
			const FVec3 CV0 = V(0) + FVec3::CrossProduct(W(0), Arm0);
			const FVec3 CV1 = V(1) + FVec3::CrossProduct(W(1), Arm1);
			const FVec3 CV = CV1 - CV0;

			ConstraintAxisLinearVelocities[(int32)EJointCylindricalPositionConstraintType::Radial] = FVec3::DotProduct(CV, RadialAxis);
		}
	}

	void FPBDJointSolver::CalculatePlanarConstraintAxisLinearVelocities(
			const int32 AxisIndex,
			const EJointMotionType AxialMotion,
			const FPBDJointSettings& JointSettings,
			FVec3& ConstraintAxisLinearVelocities) const
	{
		FVec3 Axis;
		FReal Delta;
		FPBDJointUtilities::GetPlanarAxisDelta(ConnectorRs[0], ConnectorXs[0], ConnectorXs[1], AxisIndex, Axis, Delta);

		if (Delta < 0.0f)
		{
			Delta = -Delta;
			Axis = -Axis;
		}

		const FReal Limit = (AxialMotion == EJointMotionType::Locked) ? 0 : JointSettings.LinearLimit;
		const FReal Error = Delta - Limit;

		const FVec3 DConnectorX = ConnectorXs[1] - ConnectorXs[0];
		const FVec3 Connector0Correction = DConnectorX - Axis * Error; 

		const FVec3 Arm0 = ConnectorXs[0] + Connector0Correction - Body0().P();
		const FVec3 Arm1 = ConnectorXs[1] - Body1().P();
		const FVec3 CV0 = V(0) + FVec3::CrossProduct(W(0), Arm0);
		const FVec3 CV1 = V(1) + FVec3::CrossProduct(W(1), Arm1);
		const FVec3 CV = CV1 - CV0;

		ConstraintAxisLinearVelocities[0] = FVec3::DotProduct(CV, Axis);
	}

	void FPBDJointSolver::CalculateConstraintAxisAngularVelocities(
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings,
		FVec3& ConstraintAxisAngularVelocities
		) const
	{
		bool bHasRotationConstraints =
			(JointSettings.AngularMotionTypes[0] != EJointMotionType::Free)
			|| (JointSettings.AngularMotionTypes[1] != EJointMotionType::Free)
			|| (JointSettings.AngularMotionTypes[2] != EJointMotionType::Free);
		if (!bHasRotationConstraints)
		{
			return;
		}

		// Locked axes always use hard constraints. Limited axes use hard or soft depending on settings
		EJointMotionType TwistMotion = JointSettings.AngularMotionTypes[(int32)EJointAngularConstraintIndex::Twist];
		EJointMotionType Swing1Motion = JointSettings.AngularMotionTypes[(int32)EJointAngularConstraintIndex::Swing1];
		EJointMotionType Swing2Motion = JointSettings.AngularMotionTypes[(int32)EJointAngularConstraintIndex::Swing2];

		// If the twist axes are opposing, we cannot decompose the orientation into swing and twist angles, so just give up
		const FVec3 Twist0 = ConnectorRs[0] * FJointConstants::TwistAxis();
		const FVec3 Twist1 = ConnectorRs[1] * FJointConstants::TwistAxis();
		const FReal Twist01Dot = FVec3::DotProduct(Twist0, Twist1);
		const bool bDegenerate = (Twist01Dot < Chaos_Joint_DegenerateRotationLimit);
		if (bDegenerate)
		{
			UE_LOGF(LogChaosJoint, VeryVerbose, " Degenerate rotation at Swing %f deg", FMath::RadiansToDegrees(FMath::Acos(Twist01Dot)));
		}

		// Apply twist constraint
		// NOTE: Cannot calculate twist angle at 180degree swing
		if (SolverSettings.bEnableTwistLimits && !bDegenerate)
		{
			if (TwistMotion == EJointMotionType::Limited)
			{
				CalculateTwistConstraintAxisAngularVelocities(JointSettings, ConstraintAxisAngularVelocities);
			}
		}

		// Apply swing constraints
		// NOTE: Cannot separate swing angles at 180degree swing (but we can still apply locks)
		if (SolverSettings.bEnableSwingLimits)
		{
			if ((Swing1Motion == EJointMotionType::Limited) && (Swing2Motion == EJointMotionType::Limited))
			{
				CalculateConeConstraintAxisAngularVelocities(JointSettings, ConstraintAxisAngularVelocities);
			}
			else if ((Swing1Motion == EJointMotionType::Limited) && (Swing2Motion == EJointMotionType::Locked))
			{
				if (!bDegenerate)
				{
					CalculateSwingConstraintAxisAngularVelocities(SolverSettings, JointSettings, EJointAngularConstraintIndex::Swing1, ConstraintAxisAngularVelocities);
				}
			}
			else if ((Swing1Motion == EJointMotionType::Limited) && (Swing2Motion == EJointMotionType::Free))
			{
				if (!bDegenerate)
				{
					CalculateDualConeSwingConstraintAxisAngularVelocities(JointSettings, EJointAngularConstraintIndex::Swing1, ConstraintAxisAngularVelocities);
				}
			}
			else if ((Swing1Motion == EJointMotionType::Locked) && (Swing2Motion == EJointMotionType::Limited))
			{
				if (!bDegenerate)
				{
					CalculateSwingConstraintAxisAngularVelocities(SolverSettings, JointSettings, EJointAngularConstraintIndex::Swing2, ConstraintAxisAngularVelocities);
				}
			}
			else if ((Swing1Motion == EJointMotionType::Locked) && (Swing2Motion == EJointMotionType::Locked))
			{
			}
			else if ((Swing1Motion == EJointMotionType::Locked) && (Swing2Motion == EJointMotionType::Free))
			{
			}
			else if ((Swing1Motion == EJointMotionType::Free) && (Swing2Motion == EJointMotionType::Limited))
			{
				if (!bDegenerate)
				{
					CalculateDualConeSwingConstraintAxisAngularVelocities(JointSettings, EJointAngularConstraintIndex::Swing2, ConstraintAxisAngularVelocities);
				}
			}
			else if ((Swing1Motion == EJointMotionType::Free) && (Swing2Motion == EJointMotionType::Locked))
			{
			}
			else if ((Swing1Motion == EJointMotionType::Free) && (Swing2Motion == EJointMotionType::Free))
			{
			}
		}
	}

	void FPBDJointSolver::CalculateTwistConstraintAxisAngularVelocities(
		const FPBDJointSettings& JointSettings,
		FVec3& ConstraintAxisAngularVelocities) const
	{
		FVec3 TwistAxis;
		FReal TwistAngle;
		FPBDJointUtilities::GetTwistAxisAngle(ConnectorRs[0], ConnectorRs[1], TwistAxis, TwistAngle);

		if (TwistAngle < 0.0f)
		{
			TwistAxis = -TwistAxis;
		}
		ConstraintAxisAngularVelocities[(int32)EJointAngularConstraintIndex::Twist] = FVec3::DotProduct(W(1) - W(0), TwistAxis);
	}

	void FPBDJointSolver::CalculateConeConstraintAxisAngularVelocities(
			const FPBDJointSettings& JointSettings,
			FVec3& ConstraintAxisAngularVelocities) const
	{
		FVec3 SwingAxisLocal;
		FReal DSwingAngle = 0.0f;
		const FReal Swing1Limit = JointSettings.AngularLimits[(int32)EJointAngularConstraintIndex::Swing1];
		const FReal Swing2Limit = JointSettings.AngularLimits[(int32)EJointAngularConstraintIndex::Swing2];
		FPBDJointUtilities::GetEllipticalConeAxisErrorLocal(ConnectorRs[0], ConnectorRs[1], Swing2Limit, Swing1Limit, SwingAxisLocal, DSwingAngle);
		SwingAxisLocal.SafeNormalize();
		const FVec3 SwingAxis = ConnectorRs[0] * SwingAxisLocal;
		ConstraintAxisAngularVelocities[(int32)EJointAngularConstraintIndex::Swing1] = FVec3::DotProduct(W(1) - W(0), SwingAxis);	
	}

	void FPBDJointSolver::CalculateDualConeSwingConstraintAxisAngularVelocities(
			const FPBDJointSettings& JointSettings,
			const EJointAngularConstraintIndex SwingConstraintIndex,
			FVec3& ConstraintAxisAngularVelocities) const
	{
		FVec3 SwingAxis;
		FReal SwingAngle;
		FPBDJointUtilities::GetDualConeSwingAxisAngle(ConnectorRs[0], ConnectorRs[1], SwingConstraintIndex, SwingAxis, SwingAngle);
		ConstraintAxisAngularVelocities[(int32)SwingConstraintIndex] = FVec3::DotProduct(W(1) - W(0), SwingAxis);
	}

	void FPBDJointSolver::CalculateSwingConstraintAxisAngularVelocities(
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings,
			const EJointAngularConstraintIndex SwingConstraintIndex,
			FVec3& ConstraintAxisAngularVelocities) const
	{
		FVec3 SwingAxis;
		FReal SwingAngle;
		FPBDJointUtilities::GetSwingAxisAngle(ConnectorRs[0], ConnectorRs[1], SolverSettings.SwingTwistAngleTolerance, SwingConstraintIndex, SwingAxis, SwingAngle);
		ConstraintAxisAngularVelocities[(int32)SwingConstraintIndex] = FVec3::DotProduct(W(1) - W(0), SwingAxis);
	}
}
