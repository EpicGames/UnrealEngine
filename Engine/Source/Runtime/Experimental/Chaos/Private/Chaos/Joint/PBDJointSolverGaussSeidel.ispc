// Copyright Epic Games, Inc. All Rights Reserved.

#define EXPLICIT_VECTOR4 1

#include "Math/Vector.isph"
#include "Math/Quat.isph"
#include "Math/Transform.isph"
#include "Chaos/Matrix33.isph"

static const uniform int32 MaxConstrainedBodies = 2;

struct FPBDJointSolver
{
	// Local-space constraint settings
	FTransform XLs[MaxConstrainedBodies];		// Local-space joint connector transforms
	FVector InvILs[MaxConstrainedBodies];		// Local-space inverse inertias
	FReal InvMs[MaxConstrainedBodies];			// Inverse masses

	// World-space constraint state
	FVector Xs[MaxConstrainedBodies];			// World-space joint connector positions
	FVector4 Rs[MaxConstrainedBodies];			// World-space joint connector rotations

	// World-space body state
	FVector Ps[MaxConstrainedBodies];			// World-space particle CoM positions
	FVector4 Qs[MaxConstrainedBodies];			// World-space particle CoM rotations
	FVector Vs[MaxConstrainedBodies];			// World-space particle CoM velocities
	FVector Ws[MaxConstrainedBodies];			// World-space particle CoM angular velocities
	FMatrix33 InvIs[MaxConstrainedBodies];		// World-space inverse inertias

	// XPBD Previous iteration world-space body state
	FVector InitPs[MaxConstrainedBodies];		// World-space particle positions
	FVector4 InitQs[MaxConstrainedBodies];		// World-space particle rotations
	FVector InitXs[MaxConstrainedBodies];		// World-space joint connector positions
	FVector4 InitRs[MaxConstrainedBodies];		// World-space joint connector rotations

	// Accumulated Impulse and AngularImpulse (Impulse * Dt since they are mass multiplied position corrections)
	FVector NetLinearImpulse;
	FVector NetAngularImpulse;

	FVector LinearHardLambda;
	FVector AngularHardLambda;

	// XPBD Accumulators (net impulse for each soft constraint/drive)
	FReal LinearSoftLambda;
	FReal TwistSoftLambda;
	FReal SwingSoftLambda;
	FVector LinearDriveLambdas;
	FVector RotationDriveLambdas;
	
	// Solver stiffness - increased over iterations for stability
	// @todo(chaos): remove Stiffness from SolverSettings (because it is not a solver constant)
	FReal SolverStiffness;

	// Constraint padding which can act something like a velocity constraint (for restitution)
	FVector LinearConstraintPadding;
	FVector AngularConstraintPadding;

	// Tolerances below which we stop solving
	FReal PositionTolerance;					// Distance error below which we consider a constraint or drive solved
	FReal AngleTolerance;						// Angle error below which we consider a constraint or drive solved

	FVector LastPs[MaxConstrainedBodies];		// Positions at the beginning of the iteration
	FVector4 LastQs[MaxConstrainedBodies];		// Rotations at the beginning of the iteration
	int32 NumActiveConstraints;					// The number of active constraints and drives in the last iteration (-1 initial value)
	bool bIsActive;								// Whether any constraints actually moved any bodies in last iteration
};

export uniform int32 SizeofFPBDJointSolver()
{
	return sizeof(uniform FPBDJointSolver);
}

static inline uniform FMatrix33 ComputeWorldSpaceInertia(const uniform FVector4 &Q, const uniform FVector &I)
{
	const uniform FMatrix33 QM = MakeQuatRotationTranslationMatrix(Q);
	const uniform FMatrix33 L = SetMatrix33(I.V[0], I.V[1], I.V[2]);
	return MultiplyAB(QM, MultiplyABt(L, QM));
}

static inline uniform FMatrix33 ComputeJointFactorMatrix(const uniform FVector& V, const uniform FMatrix33& M, const uniform FReal Im)
{
	// Rigid objects rotational contribution to the impulse.
	// Vx*M*VxT+Im
	const uniform FVector V221 = SetVector(-V.V[2], V.V[2], -V.V[1]);
	const uniform FVector V222 = SetVector(-V.V[2], -V.V[2], -V.V[2]);
	const uniform FVector M544 = SetVector(M.M[5], M.M[4], M.M[4]);
	const uniform FVector V111 = SetVector(V.V[1], V.V[1], V.V[1]);
	const uniform FVector M988 = SetVector(M.M[9], M.M[8], M.M[8]);
	const uniform FVector V100 = SetVector(V.V[1], -V.V[0], V.V[0]);
	const uniform FVector M995 = SetVector(M.M[9], M.M[9], M.M[5]);
	const uniform FVector M10109 = SetVector(M.M[10], M.M[10], M.M[9]);
	const uniform FVector IM = SetVector(Im, 0, 0);

	const uniform FVector R1 = (V221 * (V222 * M544 + V111 * M988)) + (V100 * (V222 * M995 + V111 * M10109)) + IM;

	const uniform FVector V211 = SetVector(V.V[2], -V.V[1], -V.V[1]);
	const uniform FVector V221_2 = SetVector(V.V[2], V.V[2], -V.V[1]);
	const uniform FVector M000 = SetVector(M.M[0], M.M[0], M.M[0]);
	const uniform FVector V000 = SetVector(-V.V[0], -V.V[0], V.V[0]);
	const uniform FVector M884 = SetVector(M.M[8], M.M[8], M.M[4]);
	const uniform FVector V000_2 = SetVector(-V.V[0], V.V[0], V.V[0]);
	const uniform FVector M844 = SetVector(M.M[8], M.M[4], M.M[4]);
	const uniform FVector M1095 = SetVector(M.M[10], M.M[9], M.M[5]);
	const uniform FVector IM_2 = SetVector(Im, 0, Im);

	const uniform FVector R2 = (V211 * (V221_2 * M000 + V000 * M884)) + (V000_2 * (V221_2 * M844 + V000 * M1095)) + IM_2;

	return SetMatrix33(R1.V[0], R1.V[1], R1.V[2], R2.V[0], R2.V[1], R2.V[2]);
}

static inline uniform FVector CalculateAngularVelocity(const uniform FVector4 &R0, const uniform FVector4 &R1, const uniform FReal Dt)
{
	const uniform FVector4 R1Shortest = VectorQuaternionEnforceShortestArcWith(R1, R0);

	// W = 2 * dQ/dT * Qinv
	const uniform FVector4 DRDt = (R1Shortest - R0) * (ONE / Dt);
	const uniform FVector4 RInv = QuatInverse(R0);
	const uniform FVector4 W = VectorQuaternionMultiply2(DRDt, RInv) * TWO;

	return SetVector(W);
}

static inline uniform FVector CalculateVelocity(const uniform FVector& P0, const uniform FVector& P1, const uniform FReal Dt)
{
	return (P1 - P0) / Dt;
}

static inline void ApplyPositionDelta(
	uniform FPBDJointSolver * uniform M,
	const uniform int Index,
	const uniform FVector& DP)
{
	M->Ps[Index] = M->Ps[Index] + DP;
}

static inline void ApplyPositionDelta(
	uniform FPBDJointSolver * uniform M,
	const uniform FVector& DP0,
	const uniform FVector& DP1)
{
	M->Ps[0] = M->Ps[0] + DP0;
	M->Ps[1] = M->Ps[1] + DP1;
}

static inline void ApplyRotationDelta(
	uniform FPBDJointSolver * uniform M,
	const uniform int Index,
	const uniform FVector& InDR)
{
	const uniform FVector4 Qs = M->Qs[Index];
	const uniform FVector4 DR = SetVector4(InDR, 0);

	const uniform FVector4 DQ = VectorQuaternionMultiply2(DR, Qs) * ONE_HALF;

	const uniform FVector4 QsNorm = VectorNormalizeQuaternion(Qs + DQ);

	M->Qs[Index] = QsNorm;

	const uniform FVector4 Qs1Shortest = VectorQuaternionEnforceShortestArcWith(M->Qs[1], M->Qs[0]);
	M->Qs[1] = Qs1Shortest;

	M->Xs[Index] = M->Ps[Index] + SetVector(VectorQuaternionRotateVector(M->Qs[Index], M->XLs[Index].Translation));
	M->Rs[Index] = VectorQuaternionMultiply2(M->Qs[Index], M->XLs[Index].Rotation);

	const uniform FVector4 Rs1Shortest = VectorQuaternionEnforceShortestArcWith(M->Rs[1], M->Rs[0]);
	M->Rs[1] = Rs1Shortest;

	M->InvIs[Index] = ComputeWorldSpaceInertia(M->Qs[Index], M->InvILs[Index]);
}

static inline void ApplyRotationDelta(
	uniform FPBDJointSolver * uniform M,
	const uniform FVector& InDR0,
	const uniform FVector& InDR1)
{
	if (M->InvMs[0] > 0)
	{
		const uniform FVector4 Qs0 = M->Qs[0];
		const uniform FVector4 DR0 = SetVector4(InDR0, 0);
		const uniform FVector4 DQ0 = VectorQuaternionMultiply2(DR0, Qs0) * ONE_HALF;
		const uniform FVector4 Q0 = VectorNormalizeQuaternion(Qs0 + DQ0);
		M->Qs[0] = Q0;
		M->Xs[0] = M->Ps[0] + SetVector(VectorQuaternionRotateVector(Q0, M->XLs[0].Translation));
		M->Rs[0] = VectorQuaternionMultiply2(Q0, M->XLs[0].Rotation);
		M->InvIs[0] = ComputeWorldSpaceInertia(Q0, M->InvILs[0]);
	}

	if (M->InvMs[1] > 0)
	{
		const uniform FVector4 Qs1 = M->Qs[1];
		const uniform FVector4 DR1 = SetVector4(InDR1, 0);
		const uniform FVector4 DQ1 = VectorQuaternionMultiply2(DR1, Qs1) * ONE_HALF;
		const uniform FVector4 Q1 = VectorNormalizeQuaternion(Qs1 + DQ1);
		M->Qs[1] = Q1;
		M->Xs[1] = M->Ps[1] + SetVector(VectorQuaternionRotateVector(Q1, M->XLs[1].Translation));
		M->Rs[1] = VectorQuaternionMultiply2(Q1, M->XLs[1].Rotation);
		M->InvIs[1] = ComputeWorldSpaceInertia(Q1, M->InvILs[1]);
	}

	M->Qs[1] = VectorQuaternionEnforceShortestArcWith(M->Qs[1], M->Qs[0]);
	M->Rs[1] = VectorQuaternionEnforceShortestArcWith(M->Rs[1], M->Rs[0]);
}

export void ApplyRotationDelta2(
	uniform FPBDJointSolver * uniform M,
	const uniform FVector& DR0,
	const uniform FVector& DR1)
{
	ApplyRotationDelta(M, DR0, DR1);
}

static inline void ApplyDelta(
	uniform FPBDJointSolver * uniform M,
	const uniform int Index,
	const uniform FVector& DP,
	const uniform FVector& DR)
{
	ApplyPositionDelta(M, Index, DP);
	ApplyRotationDelta(M, Index, DR);
}

static inline void ApplyVelocityDelta(
	uniform FPBDJointSolver * uniform M,
	const uniform FVector& DV0,
	const uniform FVector& DW0,
	const uniform FVector& DV1,
	const uniform FVector& DW1)
{
	M->Vs[0] = M->Vs[0] + DV0;
	M->Vs[1] = M->Vs[1] + DV1;
	M->Ws[0] = M->Ws[0] + DW0;
	M->Ws[1] = M->Ws[1] + DW1;
}


static inline void ApplyVelocityDelta(
	uniform FPBDJointSolver * uniform M,
	const uniform int Index,
	const uniform FVector& DV,
	const uniform FVector& DW)
{
	M->Vs[Index] = M->Vs[Index] + DV;
	M->Ws[Index] = M->Ws[Index] + DW;
}

export void ApplyPositionConstraintSoft(
	uniform FPBDJointSolver * uniform M,
	const uniform FReal Dt,
	const uniform FReal JointStiffness,
	const uniform FReal JointDamping,
	const uniform bool bAccelerationMode,
	const uniform FVector& Axis,
	const uniform FReal Delta,
	const uniform FReal TargetVel,
	uniform FReal& Lambda)
{
	// Joint-space inverse mass
	const uniform FVector AngularAxis0 = VectorCross(M->Xs[0] - M->Ps[0], Axis);
	const uniform FVector AngularAxis1 = VectorCross(M->Xs[1] - M->Ps[1], Axis);
	const uniform FVector IA0 = Multiply(M->InvIs[0], AngularAxis0);
	const uniform FVector IA1 = Multiply(M->InvIs[1], AngularAxis1);
	const uniform FReal II0 = VectorDot(AngularAxis0, IA0);
	const uniform FReal II1 = VectorDot(AngularAxis1, IA1);
	const uniform FReal II = (M->InvMs[0] + II0 + M->InvMs[1] + II1);

	uniform FReal VelDt = 0;
	if (JointDamping > KINDA_SMALL_NUMBER)
	{
		const uniform FVector V0Dt = CalculateVelocity(M->InitXs[0], M->Xs[0], ONE);
		const uniform FVector V1Dt = CalculateVelocity(M->InitXs[1], M->Xs[1], ONE);
		VelDt = TargetVel * Dt + VectorDot(V0Dt - V1Dt, Axis);
	}

	const uniform FReal SpringMassScale = (bAccelerationMode) ? ONE / (M->InvMs[0] + M->InvMs[1]) : ONE;
	const uniform FReal S = SpringMassScale * JointStiffness * Dt * Dt;
	const uniform FReal D = SpringMassScale * JointDamping * Dt;
	const uniform FReal Multiplier = ONE / ((S + D) * II + ONE);
	const uniform FReal DLambda = M->SolverStiffness * Multiplier * (S * Delta - D * VelDt - Lambda);
	
	const uniform FVector DP0 = (M->InvMs[0] * DLambda) * Axis;
	const uniform FVector DP1 = (-M->InvMs[1] * DLambda) * Axis;
	const uniform FVector DR0 = DLambda * Multiply(M->InvIs[0], AngularAxis0);
	const uniform FVector DR1 = -DLambda * Multiply(M->InvIs[1], AngularAxis1);

	ApplyPositionDelta(M, DP0, DP1);
	ApplyRotationDelta(M, DR0, DR1);

	Lambda += DLambda;
	M->NetLinearImpulse = M->NetLinearImpulse + DLambda * Axis;
}

// See "XPBD: Position-Based Simulation of Compliant Constrained Dynamics"
export void ApplyRotationConstraintSoftKD(
	uniform FPBDJointSolver * uniform M,
	const uniform int KIndex,
	const uniform int DIndex,
	const uniform FReal Dt,
	const uniform FReal JointStiffness,
	const uniform FReal JointDamping,
	const uniform bool bAccelerationMode,
	const uniform FVector& Axis,
	const uniform FReal Angle,
	const uniform FReal AngVelTarget,
	uniform FReal& Lambda)
{
	// World-space inverse mass
	const uniform FVector IA1 = Multiply(M->InvIs[DIndex], Axis);

	// Joint-space inverse mass
	uniform FReal II1 = VectorDot(Axis, IA1);
	const uniform FReal II = II1;

	// Damping angular velocity
	uniform FReal AngVelDt = 0;
	if (JointDamping > KINDA_SMALL_NUMBER)
	{
		const uniform FVector W0Dt = CalculateAngularVelocity(M->InitRs[KIndex], M->Rs[KIndex], ONE);
		const uniform FVector W1Dt = CalculateAngularVelocity(M->InitRs[DIndex], M->Rs[DIndex], ONE);
		AngVelDt = AngVelTarget * Dt + VectorDot(Axis, W0Dt - W1Dt);
	}

	const uniform FReal SpringMassScale = (bAccelerationMode) ? ONE / II : ONE;
	const uniform FReal S = SpringMassScale * JointStiffness * Dt * Dt;
	const uniform FReal D = SpringMassScale * JointDamping * Dt;
	const uniform FReal Multiplier = ONE / ((S + D) * II + ONE);
	const uniform FReal DLambda = M->SolverStiffness * Multiplier * (S * Angle - D * AngVelDt - Lambda);

	//const FVec3 DR1 = IA1 * -DLambda;
	const uniform FVector DR1 = Axis * -(DLambda * II1);

	ApplyRotationDelta(M, DIndex, DR1);

	Lambda += DLambda;
	M->NetAngularImpulse = M->NetAngularImpulse + (KIndex == 0 ? 1 : -1) * DLambda * Axis;
}

export void ApplyRotationConstraintSoftDD(
	uniform FPBDJointSolver * uniform M,
	const uniform FReal Dt,
	const uniform FReal JointStiffness,
	const uniform FReal JointDamping,
	const uniform bool bAccelerationMode,
	const uniform FVector& Axis,
	const uniform FReal Angle,
	const uniform FReal AngVelTarget,
	uniform FReal& Lambda)
{
	// World-space inverse mass
	const uniform FVector IA0 = Multiply(M->InvIs[0], Axis);
	const uniform FVector IA1 = Multiply(M->InvIs[1], Axis);

	// Joint-space inverse mass
	uniform FReal II0 = VectorDot(Axis, IA0);
	uniform FReal II1 = VectorDot(Axis, IA1);
	const uniform FReal II = (II0 + II1);

	// Damping angular velocity
	uniform FReal AngVelDt = 0;
	if (JointDamping > KINDA_SMALL_NUMBER)
	{
		const uniform FVector W0Dt = CalculateAngularVelocity(M->InitRs[0], M->Rs[0], ONE);
		const uniform FVector W1Dt = CalculateAngularVelocity(M->InitRs[1], M->Rs[1], ONE);
		AngVelDt = AngVelTarget * Dt + VectorDot(Axis, W0Dt - W1Dt);
	}

	const uniform FReal SpringMassScale = (bAccelerationMode) ? ONE / II : ONE;
	const uniform FReal S = SpringMassScale * JointStiffness * Dt * Dt;
	const uniform FReal D = SpringMassScale * JointDamping * Dt;
	const uniform FReal Multiplier = ONE / ((S + D) * II + ONE);
	const uniform FReal DLambda = M->SolverStiffness * Multiplier * (S * Angle - D * AngVelDt - Lambda);

	//const FVec3 DR0 = IA0 * DLambda;
	//const FVec3 DR1 = IA1 * -DLambda;
	const uniform FVector DR0 = Axis * (DLambda * II0);
	const uniform FVector DR1 = Axis * -(DLambda * II1);

	ApplyRotationDelta(M, DR0, DR1);

	Lambda += DLambda;
	M->NetAngularImpulse = M->NetAngularImpulse + DLambda * Axis;
}


export void ApplyPointPositionConstraintKD(
	uniform FPBDJointSolver * uniform M,
	const uniform int KIndex,
	const uniform int DIndex,
	const uniform FVector &CX,
	const uniform FReal InStiffness)
{
	// Calculate constraint correction
	const uniform FMatrix33 M1 = ComputeJointFactorMatrix(M->Xs[DIndex] - M->Ps[DIndex], M->InvIs[DIndex], M->InvMs[DIndex]);
	const uniform FMatrix33 MI = MatrixInverse(M1);
	const uniform FVector DX = InStiffness * Multiply(MI, CX);

	// Apply constraint correction
	const uniform FVector DP1 = (DX * -M->InvMs[DIndex]);
	const uniform FVector DR1 = Multiply(M->InvIs[DIndex], VectorCross(M->Xs[DIndex] - M->Ps[DIndex], VectorNegate(DX)));

	ApplyDelta(M, DIndex, DP1, DR1);

	M->NetLinearImpulse = M->NetLinearImpulse + ((KIndex == 0) ? DX : VectorNegate(DX));
}

export void ApplyPointPositionConstraintDD(
	uniform FPBDJointSolver * uniform M,
	const uniform FVector &CX,
	const uniform FReal InStiffness)
{
	// Calculate constraint correction
	const uniform FMatrix33 M0 = ComputeJointFactorMatrix(M->Xs[0] - M->Ps[0], M->InvIs[0], M->InvMs[0]);
	const uniform FMatrix33 M1 = ComputeJointFactorMatrix(M->Xs[1] - M->Ps[1], M->InvIs[1], M->InvMs[1]);
	const uniform FMatrix33 MI = MatrixInverse(AddAB(M0, M1));
	const uniform FVector DX = InStiffness * Multiply(MI, CX);

	// Apply constraint correction
	const uniform FVector DP0 = (DX * M->InvMs[0]);
	const uniform FVector DP1 = (DX * -M->InvMs[1]);
	const uniform FVector DR0 = Multiply(M->InvIs[0], VectorCross(M->Xs[0] - M->Ps[0], DX));
	const uniform FVector DR1 = Multiply(M->InvIs[1], VectorCross(M->Xs[1] - M->Ps[1], VectorNegate(DX)));

	ApplyPositionDelta(M, DP0, DP1);
	ApplyRotationDelta(M, DR0, DR1);

	M->NetLinearImpulse = M->NetLinearImpulse + DX;
}

