// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/KinematicGeometryParticles.h"
#include "Chaos/ParticleHandle.h"

namespace Chaos
{
	template<class T, int d, EGeometryParticlesSimType SimType>
	const typename TKinematicGeometryParticlesImp<T, d, SimType>::THandleType* TKinematicGeometryParticlesImp<T, d, SimType>::Handle(int32 Index) const
	{
		checkf(SimType == EGeometryParticlesSimType::RigidBodySim, TEXT("Handles require a rigid body sim type"));
		return static_cast<const TKinematicGeometryParticlesImp<T, d, SimType>::THandleType*>(TGeometryParticlesImp<T, d, SimType>::Handle(Index));
	}

	template<class T, int d, EGeometryParticlesSimType SimType>
	typename TKinematicGeometryParticlesImp<T, d, SimType>::THandleType* Chaos::TKinematicGeometryParticlesImp<T, d, SimType>::Handle(int32 Index)
	{
		//Todo: turn on static_assert CHAOS_PARTICLEHANDLE_TODO
		//static_assert(SimType == EGeometryParticlesSimType::RigidBodySim, "Handles require a rigid body sim type");
		return static_cast<TKinematicGeometryParticlesImp<T, d, SimType>::THandleType*>(TGeometryParticlesImp<T, d, SimType>::Handle(Index));
	}

}

template class Chaos::TKinematicGeometryParticlesImp<Chaos::FReal, 3, Chaos::EGeometryParticlesSimType::RigidBodySim>;
template class Chaos::TKinematicGeometryParticlesImp<Chaos::FReal, 3, Chaos::EGeometryParticlesSimType::Other>;
class FKinematicTarget;

// Re-enable when double precision is able to compile
//template Chaos::TKinematicGeometryParticlesImp<double, 3>;
