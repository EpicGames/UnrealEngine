// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/Levelset.h"
#include "Containers/Queue.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Async/ParallelFor.h"

#include <algorithm>
#include <vector>
#include "Chaos/ErrorReporter.h"
#include "Chaos/MassProperties.h"
#include "Chaos/TriangleMesh.h"
#include "Chaos/Sphere.h"
#include "Chaos/Box.h"
#include "Chaos/Capsule.h"
#include "Chaos/Convex.h"
#include "Chaos/GeometryQueries.h"

int32 OutputFailedLevelSetDebugData = 0;
FAutoConsoleVariableRef CVarOutputFailedLevelSetDebugData(TEXT("p.LevelSetOutputFailedDebugData"), OutputFailedLevelSetDebugData, TEXT("Output debug obj files for level set and mesh when error tolerances are too high"));

int32 FailureOnHighError = 0;
FAutoConsoleVariableRef CVarFailureOnHighError(TEXT("p.LevelSetFailureOnHighError"), FailureOnHighError, TEXT("Set level sets with high error to null in the solver"));

Chaos::FRealSingle AvgDistErrorTolerance = 1.f;
FAutoConsoleVariableRef CVarAvgDistErrorTolerance(TEXT("p.LevelSetAvgDistErrorTolerance"), AvgDistErrorTolerance, TEXT("Error tolerance for average distance between the triangles and generated levelset.  Note this is a fraction of the average bounding box dimensions."));

Chaos::FRealSingle MaxDistErrorTolerance = 1.f;
FAutoConsoleVariableRef CVarMaxDistErrorTolerance(TEXT("p.LevelSetMaxDistErrorTolerance"), MaxDistErrorTolerance, TEXT("Max error for the highest error triangle generated from a levelset.  Note this is a fraction of the average bounding box dimensions."));

Chaos::FRealSingle AvgAngleErrorTolerance = 1.;
FAutoConsoleVariableRef CVarAvgAngleErrorTolerance(TEXT("p.LevelSetAvgAngleErrorTolerance"), AvgAngleErrorTolerance, TEXT("Average error in of the mesh normal and computed normal on the level set."));

int32 NumOverlapSphereSamples = 16;
FAutoConsoleVariableRef CVarNumOverlapSphereSamples(TEXT("p.LevelsetOverlapSphereSamples"), NumOverlapSphereSamples, TEXT("Number of spiral points to generate for levelset-sphere overlaps"));

int32 NumOverlapCapsuleSamples = 24;
FAutoConsoleVariableRef CVarNumOverlapCapsuleSamples(TEXT("p.LevelsetOverlapCapsuleSamples"), NumOverlapCapsuleSamples, TEXT("Number of spiral points to generate for levelset-capsule overlaps"));

#define MAX_CLAMP(a, comp, b) (a >= comp ? b : a)
#define MIN_CLAMP(a, comp, b) (a < comp ? b : a)
#define RANGE_CLAMP(a, comp, b) ((a < 0 || comp <= a) ? b : a)

namespace Chaos
{
FLevelSet::FLevelSet(FErrorReporter& ErrorReporter, const TUniformGrid<FReal, 3>& InGrid, const FParticles& InParticles, const FTriangleMesh& Mesh, const int32 BandWidth)
    : FImplicitObject(EImplicitObject::HasBoundingBox, ImplicitObjectType::LevelSet)
    , MGrid(InGrid)
    , MPhi(MGrid)
    , MNormals(MGrid)
    , MLocalBoundingBox(MGrid.MinCorner(), MGrid.MaxCorner())
    , MBandWidth(BandWidth)
{
	check(MGrid.Counts()[0] > 1 && MGrid.Counts()[1] > 1 && MGrid.Counts()[2] > 1);
	check(Mesh.GetSurfaceElements().Num());

	const TArray<FVec3> Normals = 
		Mesh.GetFaceNormals(
			InParticles, 
			false);			// Don't fail if the mesh has small faces
	if (Normals.Num() == 0)
	{
		ErrorReporter.ReportError(TEXT("Normals came back empty."));
		return;
	}

	TArrayND<bool, 3> BlockedFaceX(MGrid.Counts());
	TArrayND<bool, 3> BlockedFaceY(MGrid.Counts());
	TArrayND<bool, 3> BlockedFaceZ(MGrid.Counts());
	TArray<TVec3<int32>> InterfaceIndices;
	if (!ComputeDistancesNearZeroIsocontour(ErrorReporter, InParticles, Normals, Mesh, BlockedFaceX, BlockedFaceY, BlockedFaceZ, InterfaceIndices))
	{
		ErrorReporter.ReportError(TEXT("Error calling FLevelSet::ComputeDistancesNearZeroIsocontour"));
		return;
	}
	FReal StoppingDistance = static_cast<FReal>(MBandWidth) * MGrid.Dx().Max();
	if (StoppingDistance != 0)
	{
		for (int32 i = 0; i < MGrid.Counts().Product(); ++i)
		{
			MPhi[i] = FGenericPlatformMath::Min(MPhi[i], StoppingDistance);
		}
	}
	CorrectSign(BlockedFaceX, BlockedFaceY, BlockedFaceZ, InterfaceIndices);
	FillWithFastMarchingMethod(StoppingDistance, InterfaceIndices);
	if (StoppingDistance != 0)
	{
		for (int32 i = 0; i < MGrid.Counts().Product(); ++i)
		{
			if (FGenericPlatformMath::Abs(MPhi[i]) > StoppingDistance)
			{
				MPhi[i] = MPhi[i] > 0 ? StoppingDistance : -StoppingDistance;
			}
		}
	}
	//ComputeNormals(InParticles, Mesh, InterfaceIndices);
	ComputeNormals();
	ComputeConvexity(InterfaceIndices);

	// Check newly created level set values for inf/nan
	bool ValidLevelSet = CheckData(ErrorReporter, InParticles, Mesh, Normals);
}

FLevelSet::FLevelSet(FErrorReporter& ErrorReporter, const TUniformGrid<FReal, 3>& InGrid, const FImplicitObject& InObject, const int32 BandWidth, const bool bUseObjectPhi)
    : FImplicitObject(EImplicitObject::HasBoundingBox, ImplicitObjectType::LevelSet)
    , MGrid(InGrid)
    , MPhi(MGrid)
    , MNormals(MGrid)
    , MLocalBoundingBox(MGrid.MinCorner(), MGrid.MaxCorner())
    , MOriginalLocalBoundingBox(InObject.BoundingBox())
    , MBandWidth(BandWidth)
{
	check(MGrid.Counts()[0] > 1 && MGrid.Counts()[1] > 1 && MGrid.Counts()[2] > 1);
	const auto& Counts = MGrid.Counts();
	if (bUseObjectPhi)
	{
		for (int32 i = 0; i < Counts.Product(); ++i)
		{
			MPhi[i] = InObject.SignedDistance(MGrid.Center(i));
		}
		ComputeNormals();
		return;
	}
	TArrayND<FReal, 3> ObjectPhi(MGrid);
	for (int32 i = 0; i < Counts.Product(); ++i)
	{
		ObjectPhi[i] = InObject.SignedDistance(MGrid.Center(i));
	}
	TArray<TVec3<int32>> InterfaceIndices;
	ComputeDistancesNearZeroIsocontour(InObject, ObjectPhi, InterfaceIndices);
	FReal StoppingDistance = static_cast<FReal>(MBandWidth) * MGrid.Dx().Max();
	if (StoppingDistance != 0)
	{
		for (int32 i = 0; i < MGrid.Counts().Product(); ++i)
		{
			MPhi[i] = FGenericPlatformMath::Min(MPhi[i], StoppingDistance);
		}
	}
	// Correct Sign
	for (int32 i = 0; i < MGrid.Counts().Product(); ++i)
	{
		MPhi[i] *= FMath::Sign(ObjectPhi[i]);
	}
	FillWithFastMarchingMethod(StoppingDistance, InterfaceIndices);
	if (StoppingDistance != 0)
	{
		for (int32 i = 0; i < MGrid.Counts().Product(); ++i)
		{
			if (FGenericPlatformMath::Abs(MPhi[i]) > StoppingDistance)
			{
				MPhi[i] = MPhi[i] > 0 ? StoppingDistance : -StoppingDistance;
			}
		}
	}
	ComputeNormals();
	ComputeConvexity(InterfaceIndices);
}

#if COMPILE_WITHOUT_UNREAL_SUPPORT
FLevelSet::FLevelSet(std::istream& Stream)
    : FImplicitObject(EImplicitObject::HasBoundingBox, ImplicitObjectType::LevelSet)
    , MGrid(Stream)
    , MPhi(Stream)
    , MLocalBoundingBox(MGrid.MinCorner(), MGrid.MaxCorner())
{
	Stream.read(reinterpret_cast<char*>(&MBandWidth), sizeof(MBandWidth));
	ComputeNormals();
}
#endif
FLevelSet::FLevelSet(TUniformGrid<FReal, 3>&& Grid, TArrayND<FReal, 3>&& Phi, int32 BandWidth)
	: FImplicitObject(EImplicitObject::HasBoundingBox, ImplicitObjectType::LevelSet)
	, MGrid(MoveTemp(Grid))
	, MPhi(MoveTemp(Phi))
	, MNormals(MGrid)
	, MLocalBoundingBox(MGrid.MinCorner(), MGrid.MaxCorner())
	, MOriginalLocalBoundingBox(MLocalBoundingBox)
	, MBandWidth(BandWidth)
{
	ComputeNormals();
}

FLevelSet::FLevelSet(FLevelSet&& Other)
    : FImplicitObject(EImplicitObject::HasBoundingBox, ImplicitObjectType::LevelSet)
    , MGrid(MoveTemp(Other.MGrid))
    , MPhi(MoveTemp(Other.MPhi))
	, MNormals(MoveTemp(Other.MNormals))
    , MLocalBoundingBox(MoveTemp(Other.MLocalBoundingBox))
	, MOriginalLocalBoundingBox(MoveTemp(Other.MOriginalLocalBoundingBox))
    , MBandWidth(Other.MBandWidth)
{
}

FLevelSet::~FLevelSet()
{
}
Chaos::FImplicitObjectPtr FLevelSet::CopyGeometry() const
{
	FLevelSet* Copy = new FLevelSet();
	Copy->MGrid = MGrid;
	Copy->MPhi.Copy(MPhi);
	Copy->MNormals.Copy(MNormals);
	Copy->MLocalBoundingBox = MLocalBoundingBox;
	Copy->MOriginalLocalBoundingBox = MOriginalLocalBoundingBox;
	Copy->MBandWidth = MBandWidth;
	return Chaos::FImplicitObjectPtr(Copy);
}

Chaos::FImplicitObjectPtr FLevelSet::CopyGeometryWithScale(const FVec3& Scale) const
{
	FLevelSet* Copy = new FLevelSet();
	Copy->MGrid = MGrid;
	Copy->MPhi.Copy(MPhi);
	Copy->MNormals.Copy(MNormals);
	Copy->MLocalBoundingBox = MLocalBoundingBox;
	Copy->MOriginalLocalBoundingBox = MOriginalLocalBoundingBox;
	Copy->MBandWidth = MBandWidth;
	return MakeImplicitObjectPtr<TImplicitObjectScaled<FLevelSet>>(Copy, Scale);
}

bool FLevelSet::ComputeMassProperties(FReal& OutVolume, FVec3& OutCOM, FMatrix33& OutInertia, FRotation3& OutRotationOfMass) const
{
	FVec3 COM(0);
	TArray<TVec3<int32>> CellsWithVolume;

	const FVec3 CellExtents = MGrid.Dx();
	const FVec3 ExtentsSquared(CellExtents * CellExtents);
	const FReal CellVolume = CellExtents.Product();
	const FMatrix33 CellInertia((ExtentsSquared[1] + ExtentsSquared[2]) / (FReal)12, (ExtentsSquared[0] + ExtentsSquared[2]) / (FReal)12, (ExtentsSquared[0] + ExtentsSquared[1]) / (FReal)12);

	for (int32 X = 0; X < MGrid.Counts()[0]; ++X)
	{
		for (int32 Y = 0; Y < MGrid.Counts()[1]; ++Y)
		{
			for (int32 Z = 0; Z < MGrid.Counts()[2]; ++Z)
			{
				TVec3<int32> Cell(X, Y, Z);
				if (MPhi(Cell) < 0)
				{
					CellsWithVolume.Add(Cell);
					COM += (MGrid.Location(Cell) * CellVolume);
				}
			}
		}
	}

	const int32 NumCellsWithVolume = CellsWithVolume.Num();
	FReal Volume = static_cast<FReal>(NumCellsWithVolume) * CellVolume;
	FMatrix33 Inertia = CellInertia * (FReal)NumCellsWithVolume;
	if (Volume > 0)
	{
		COM /= Volume;
	}

	for (const TVec3<int32>& Cell : CellsWithVolume)
	{
		const FVec3 Dist = MGrid.Location(Cell) - COM;
		const FVec3 Dist2(Dist * Dist);
		{
			Inertia += FMatrix33(CellVolume * (Dist2[1] + Dist2[2]), -CellVolume * Dist[1] * Dist[0], -CellVolume * Dist[2] * Dist[0], CellVolume * (Dist2[2] + Dist2[0]), -CellVolume * Dist[2] * Dist[1], CellVolume * (Dist2[1] + Dist2[0]));
		}
	}

	OutRotationOfMass = Chaos::TransformToLocalSpace(Inertia);

	OutVolume = Volume;
	OutCOM = COM;
	OutInertia = Inertia;
	return true;
}

FReal FLevelSet::ComputeLevelSetError(const FParticles& InParticles, const TArray<FVec3>& Normals, const FTriangleMesh& Mesh, FReal& AngleError, FReal& MaxDistError)
{
	const TArray<TVec3<int32>>& Faces = Mesh.GetSurfaceElements();

	TArray<FReal> DistErrorValues;
	DistErrorValues.AddDefaulted(Mesh.GetNumElements());

	// Testing that the grid normal points generally in the same direction as the face normal
	// depends on the geometry being very clean.  It's entirely reasonable that the geometry may
	// have geometry on the interior that gets filled in on the grid, at which point the gradient
	// should no longer be beholden to the direction of the surface normals.  So this test only 
	// really makes sense for points on the zero isocontour.  But even then, I think this is more
	// of a test that the geometry is well formed and consistent than whether the grid normals
	// point the right direction.  And so, until we have time to formulate a better test, we 
	// simply test that the level set normals point rougly in the same directions as the bounding 
	// box normals, at the corners of the bounding box.  See below...

	//TArray<FReal> AngleErrorValues;
	//AngleErrorValues.AddDefaulted(Mesh.GetNumElements());

	TArray<FReal> TriangleArea;
	TriangleArea.AddDefaulted(Mesh.GetNumElements());

	FReal MaxDx = MGrid.Dx().Max();

	ParallelFor(Mesh.GetNumElements(), [&](int32 i) {
		const TVec3<int32> CurrMeshFace = Faces[i];
		const FVec3 MeshFaceCenter = (InParticles.GetX(CurrMeshFace[0]) + InParticles.GetX(CurrMeshFace[1]) + InParticles.GetX(CurrMeshFace[2])) / 3.f;

		//FVec3 GridNormal;
		//FReal phi = PhiWithNormal(MeshFaceCenter, GridNormal);
		const FReal phi = SignedDistance(MeshFaceCenter);

		// ignore triangles where the the center is more than 2 voxels inside
		// #note: this biases the statistics since what we really want to do is preprocess for interior triangles, but
		// that is difficult.  Including interior triangles for levelsets from clusters biases the stats more
		if (phi > (FReal)-2. * MaxDx)
		{
			DistErrorValues[i] = FMath::Abs(phi);

			for (int j = 0; j < 3; ++j)
			{
				DistErrorValues[i] += FMath::Abs(SignedDistance(InParticles.GetX(CurrMeshFace[j])));
			}

			// per triangle error average of 3 corners and center distance to surface according to MPhi
			DistErrorValues[i] /= 4.f;

			// angle error computed by angle between mesh face normal and level set gradient
			//FVec3 MeshFaceNormal = Normals[i];
			//MeshFaceNormal.SafeNormalize();
			//GridNormal.SafeNormalize();
			//AngleErrorValues[i] = FMath::Acos(FVec3::DotProduct(MeshFaceNormal, GridNormal));

			// triangle area used for weighted average
			TriangleArea[i] = (FReal)0.5 * sqrt(FVec3::CrossProduct(InParticles.GetX(CurrMeshFace[1]) - InParticles.GetX(CurrMeshFace[0]), InParticles.GetX(CurrMeshFace[2]) - InParticles.GetX(CurrMeshFace[0])).SizeSquared());
		}
	});

	FReal TotalDistError = (FReal)0.;
	//FReal TotalAngleError = (FReal)0.;
	FReal TotalTriangleArea = (FReal)0.;
	FReal MaxError = (FReal)-1. * FLT_MAX;
	for (int i = 0; i < Mesh.GetNumElements(); ++i)
	{
		if (DistErrorValues[i] > MaxError)
		{
			MaxError = DistErrorValues[i];
		}

		// weight the error values by the area
		TotalDistError += DistErrorValues[i] * TriangleArea[i];
		//TotalAngleError += AngleErrorValues[i] * TriangleArea[i];
		TotalTriangleArea += TriangleArea[i];
	}

	// degenerate case where total triangle area is very small
	if (TotalTriangleArea < 1e-5)
	{
		MaxDistError = MAX_flt;
		//AngleError = MAX_flt;
		return MAX_flt;
	}

	FReal AvgDistError = TotalDistError / TotalTriangleArea;

	// dist error is a percentage deviation away from geometry bounds, which
	// normalizes error metrics with respect to world space size
	FVec3 BoxExtents = MLocalBoundingBox.Extents();
	FReal AvgExtents = (BoxExtents[0] + BoxExtents[1] + BoxExtents[2]) / (FReal)3.0;

	// degenerate case where extents are very small
	if (AvgExtents < 1e-5)
	{
		MaxDistError = MAX_flt;
		//AngleError = MAX_flt;
		return MAX_flt;
	}

	AvgDistError /= AvgExtents;
	MaxDistError = MaxError / AvgExtents;

	//AngleError = TotalAngleError / TotalTriangleArea;

	// Test the normal directions at the corners of the bounding box that they
	// point outward.
	const FAABB3 BBox = BoundingBox();
	const FVec3& MinPt = BBox.Min();
	const FVec3& MaxPt = BBox.Max();
	
	bool Fail = false;
	FVec3 Pt = MinPt;
	FVec3 BoxNorm, LSNorm;
	for (int i = 0; i < 8; i++)
	{
		// i
		// 0 - (min, min, min) MinPt
		// 1 - (max, min, min)
		// 2 - (min, max, min)
		// 3 - (min, min, max)

		// 4 - (max, max, max) MaxPt
		// 5 - (min, max, max)
		// 6 - (max, min, max)
		// 7 - (max, max, min)

		if (i <= 3)
		{
			Pt = MinPt;
			if (i == 1) Pt[0] = MaxPt[0];
			else if (i == 2) Pt[1] = MaxPt[1];
			else if (i == 3) Pt[2] = MaxPt[2];
		}
		else
		{
			Pt = MaxPt;
			if (i == 5) Pt[0] = MinPt[0];
			else if (i == 6) Pt[1] = MinPt[1];
			else if (i == 7) Pt[2] = MinPt[2];
		}

		BBox.PhiWithNormal(Pt, BoxNorm);
		PhiWithNormal(Pt, LSNorm);
		const FReal Dot = Chaos::FVec3::DotProduct(BoxNorm, LSNorm);
		if (Dot < 0)
		{
			AngleError += FMath::Abs(FMath::Acos(Dot));
		}
	}

	return AvgDistError;
}

void FLevelSet::OutputDebugData(FErrorReporter& ErrorReporter, const FParticles& InParticles, const TArray<FVec3>& Normals, const FTriangleMesh& Mesh, const FString FilePrefix)
{
	TArray<FVec3> OutVerts;
	TArray<FVec3> OutNormals;
	TArray<TVec3<int32>> OutFaces;
	TArray<TVec3<int32>> Faces = Mesh.GetSurfaceElements();

	// create array of verts and faces as polygon soup
	for (int i = 0; i < Mesh.GetNumElements(); ++i)
	{
		const TVec3<int32> CurrMeshFace = Faces[i];
		int idx = OutVerts.Add(InParticles.GetX(CurrMeshFace[0]));
		OutVerts.Add(InParticles.GetX(CurrMeshFace[1]));
		OutVerts.Add(InParticles.GetX(CurrMeshFace[2]));

		OutNormals.Add(Normals[i]);
		OutNormals.Add(Normals[i]);
		OutNormals.Add(Normals[i]);

		OutFaces.Add(TVec3<int32>(idx, idx + 1, idx + 2));
	}

	// build obj file string
	FString MeshFileStr;

	for (int i = 0; i < OutVerts.Num(); ++i)
	{
		MeshFileStr += FString::Printf(TEXT("v %f %f %f %f %f %f\n"), OutVerts[i].X, OutVerts[i].Y, OutVerts[i].Z, OutNormals[i].X, OutNormals[i].Y, OutNormals[i].Z);
	}

	for (TVec3<int32> face : OutFaces)
	{
		MeshFileStr += FString::Printf(TEXT("f %d %d %d\n"), face.X + 1, face.Y + 1, face.Z + 1);
	}

	// create volume string for phi
	FString VolumeFileStr;
	FString VolumeFileStr2;
	for (int x = 0; x < MGrid.Counts().X; ++x)
	{
		for (int y = 0; y < MGrid.Counts().Y; ++y)
		{
			for (int z = 0; z < MGrid.Counts().Z; ++z)
			{
				FVec3 loc = MGrid.Location(TVec3<int32>(x, y, z));
				FReal phi = MPhi(x, y, z);
				VolumeFileStr += FString::Printf(TEXT("v %f %f %f %f %f %f\n"), loc.X, loc.Y, loc.Z, phi, phi, phi);
			}
		}
	}

	// create volume string for normal
	for (int x = 0; x < MGrid.Counts().X; ++x)
	{
		for (int y = 0; y < MGrid.Counts().Y; ++y)
		{
			for (int z = 0; z < MGrid.Counts().Z; ++z)
			{
				FVec3 loc = MGrid.Location(TVec3<int32>(x, y, z));
				FVec3 Normal = MNormals(x, y, z);
				VolumeFileStr2 += FString::Printf(TEXT("v %f %f %f %f %f %f\n"), loc.X, loc.Y, loc.Z, Normal[0], Normal[1], Normal[2]);
			}
		}
	}

	FString SaveDirectory = FPaths::ProjectSavedDir() + "/DebugLevelSet";

	// if we have the file, increment a counter
	FString PrefixToUse = FilePrefix;	

	// write out file for mesh
	{
		FString FileName = PrefixToUse + FString("Mesh.obj");
		FileName = FPaths::MakeValidFileName(FileName);

		FString AbsoluteFilePath = SaveDirectory + "/" + FileName;
		if (!FFileHelper::SaveStringToFile(MeshFileStr, *AbsoluteFilePath))
		{
			ErrorReporter.ReportError(TEXT("Cannot write mesh"));
		}
	}

	// write out volume file
	{
		FString FileName = PrefixToUse + FString("Volume.obj");
		FileName = FPaths::MakeValidFileName(FileName);

		FString AbsoluteFilePath = SaveDirectory + "/" + FileName;
		if (!FFileHelper::SaveStringToFile(VolumeFileStr, *AbsoluteFilePath))
		{
			ErrorReporter.ReportError(TEXT("Cannot write volume"));
		}
	}

	// write out aux volume file
	{
		FString FileName = PrefixToUse + FString("Volume2.obj");
		FileName = FPaths::MakeValidFileName(FileName);

		FString AbsoluteFilePath = SaveDirectory + "/" + FileName;
		if (!FFileHelper::SaveStringToFile(VolumeFileStr2, *AbsoluteFilePath))
		{
			ErrorReporter.ReportError(TEXT("Cannot write volume"));
		}
	}
}

bool FLevelSet::CheckData(FErrorReporter& ErrorReporter, const FParticles& InParticles, const FTriangleMesh& Mesh, const TArray<FVec3>& Normals)
{
	FString ObjectNamePrefixNoSpace = ErrorReporter.GetPrefix();
	ObjectNamePrefixNoSpace.RemoveSpacesInline();
	ObjectNamePrefixNoSpace = ObjectNamePrefixNoSpace.Replace(TEXT("|"), TEXT("_"));
	ObjectNamePrefixNoSpace = ObjectNamePrefixNoSpace.Replace(TEXT(":"), TEXT("_"));	
	ObjectNamePrefixNoSpace += "__";
	// loop through and check values in phi and normals
	bool hasInterior = false;
	bool hasExterior = false;
	for (int32 i = 0; i < MGrid.Counts().Product(); ++i)
	{
		if (MNormals[i].ContainsNaN() || !FMath::IsFinite(MPhi[i]) || FMath::IsNaN(MPhi[i]))
		{
			if (OutputFailedLevelSetDebugData)
			{
				OutputDebugData(ErrorReporter, InParticles, Normals, Mesh, FString::Printf(TEXT("NANS___")) + ObjectNamePrefixNoSpace);
			}

			ErrorReporter.ReportError(TEXT("NaNs were found in level set data.  Check input geometry and resolution settings."));
			return false;
		}

		hasInterior = !hasInterior ? MPhi[i] < 0.f : hasInterior;
		hasExterior = !hasExterior ? MPhi[i] > 0.f : hasExterior;
	}

	if (!hasInterior)
	{
		if (OutputFailedLevelSetDebugData)
		{
			OutputDebugData(ErrorReporter, InParticles, Normals, Mesh, FString::Printf(TEXT("NOINTERIOR___")) + ObjectNamePrefixNoSpace);
		}

		ErrorReporter.ReportError(TEXT("No interior voxels (phi < 0) defined on level set"));
		return false;
	}

	if (!hasExterior)
	{
		if (OutputFailedLevelSetDebugData)
		{
			OutputDebugData(ErrorReporter, InParticles, Normals, Mesh, FString::Printf(TEXT("NOEXTERIOR___")) + ObjectNamePrefixNoSpace);
		}

		ErrorReporter.ReportError(TEXT("No exterior voxels (phi > 0) defined on level set"));
		return false;
	}

	FReal AvgAngleError = 0;
	FReal MaxDistError = 0;
	FReal AvgDistError = ComputeLevelSetError(InParticles, Normals, Mesh, AvgAngleError, MaxDistError);

	// Report high error, but don't report it as an invalid level set
	if (AvgDistError > AvgDistErrorTolerance*MGrid.Dx().Size() || AvgAngleError > AvgAngleErrorTolerance || MaxDistError > MaxDistErrorTolerance*MGrid.Dx().Size())
	{
		if (OutputFailedLevelSetDebugData)
		{
			FString Prefix = FString::Printf(TEXT("AVGDIST_%f__MAXDIST_%f__ANGLE_%f___"), AvgDistError, MaxDistError, AvgAngleError) + ObjectNamePrefixNoSpace;
			OutputDebugData(ErrorReporter, InParticles, Normals, Mesh, Prefix);
		}

		if (FailureOnHighError)
		{
			FString ErrorStr = FString::Printf(TEXT("High error for level set: AvgDistError: %f (Max: %f*%f), MaxDistError: %f (Max: %f*%f), AvgAngleError: %f (Max: %f)"), 
				AvgDistError, AvgDistErrorTolerance, MGrid.Dx().Size(),
				MaxDistError, MaxDistErrorTolerance, MGrid.Dx().Size(),
				AvgAngleError, AvgAngleErrorTolerance);
			ErrorReporter.ReportError(*ErrorStr);
			return false;
		}
		else
		{
			UE_LOGF(LogChaos, Log, "%ls: High error for level set: AvgDistError: %f (Max: %f*%f), MaxDistError: %f (Max: %f*%f), AvgAngleError: %f (Max: %f)", 
				*ErrorReporter.GetPrefix(), 
				AvgDistError, AvgDistErrorTolerance, MGrid.Dx().Size(),
				MaxDistError, MaxDistErrorTolerance, MGrid.Dx().Size(),
				AvgAngleError, AvgAngleErrorTolerance);
		}
	}

	return true;
}

void FLevelSet::GetZeroIsosurfaceGridCellFaces(TArray<FVector3f>& Vertices, TArray<FIntVector>& Tris) const
{
	const TVector<int32, 3> Cells = MGrid.Counts();
	const FVector3d Dx(MGrid.Dx());

	for (int i = 0; i < Cells.X - 1; ++i)
	{
		for (int j = 0; j < Cells.Y - 1; ++j)
		{
			for (int k = 0; k < Cells.Z - 1; ++k)
			{
				const double Sign = FMath::Sign(MPhi(i, j, k));
				const double SignNextI = FMath::Sign(MPhi(i + 1, j, k));
				const double SignNextJ = FMath::Sign(MPhi(i, j + 1, k));
				const double SignNextK = FMath::Sign(MPhi(i, j, k + 1));

				const FVector3d CellMin = MGrid.MinCorner() + Dx * FVector3d(i, j, k);

				if (Sign > SignNextI)
				{
					const int32 V0 = Vertices.Emplace(CellMin + Dx * FVector3d(1, 0, 0));
					const int32 V1 = Vertices.Emplace(CellMin + Dx * FVector3d(1, 1, 0));
					const int32 V2 = Vertices.Emplace(CellMin + Dx * FVector3d(1, 1, 1));
					const int32 V3 = Vertices.Emplace(CellMin + Dx * FVector3d(1, 0, 1));
					Tris.Emplace(FIntVector(V0, V1, V2));
					Tris.Emplace(FIntVector(V2, V3, V0));
				}
				else if (Sign < SignNextI)
				{
					const int32 V0 = Vertices.Emplace(CellMin + Dx * FVector3d(1, 0, 0));
					const int32 V1 = Vertices.Emplace(CellMin + Dx * FVector3d(1, 1, 0));
					const int32 V2 = Vertices.Emplace(CellMin + Dx * FVector3d(1, 1, 1));
					const int32 V3 = Vertices.Emplace(CellMin + Dx * FVector3d(1, 0, 1));
					Tris.Emplace(FIntVector(V0, V2, V1));
					Tris.Emplace(FIntVector(V2, V0, V3));
				}


				if (Sign > SignNextJ)
				{
					const int32 V0 = Vertices.Emplace(CellMin + Dx * FVector3d(0, 1, 0));
					const int32 V1 = Vertices.Emplace(CellMin + Dx * FVector3d(1, 1, 0));
					const int32 V2 = Vertices.Emplace(CellMin + Dx * FVector3d(1, 1, 1));
					const int32 V3 = Vertices.Emplace(CellMin + Dx * FVector3d(0, 1, 1));
					Tris.Emplace(FIntVector(V0, V2, V1));
					Tris.Emplace(FIntVector(V2, V0, V3));
				}
				else if (Sign < SignNextJ)
				{
					const int32 V0 = Vertices.Emplace(CellMin + Dx * FVector3d(0, 1, 0));
					const int32 V1 = Vertices.Emplace(CellMin + Dx * FVector3d(1, 1, 0));
					const int32 V2 = Vertices.Emplace(CellMin + Dx * FVector3d(1, 1, 1));
					const int32 V3 = Vertices.Emplace(CellMin + Dx * FVector3d(0, 1, 1));
					Tris.Emplace(FIntVector(V0, V1, V2));
					Tris.Emplace(FIntVector(V2, V3, V0));
				}

				if (Sign > SignNextK)
				{
					const int32 V0 = Vertices.Emplace(CellMin + Dx * FVector3d(0, 0, 1));
					const int32 V1 = Vertices.Emplace(CellMin + Dx * FVector3d(1, 0, 1));
					const int32 V2 = Vertices.Emplace(CellMin + Dx * FVector3d(1, 1, 1));
					const int32 V3 = Vertices.Emplace(CellMin + Dx * FVector3d(0, 1, 1));
					Tris.Emplace(FIntVector(V0, V1, V2));
					Tris.Emplace(FIntVector(V2, V3, V0));
				}
				else if (Sign < SignNextK)
				{
					const int32 V0 = Vertices.Emplace(CellMin + Dx * FVector3d(0, 0, 1));
					const int32 V1 = Vertices.Emplace(CellMin + Dx * FVector3d(1, 0, 1));
					const int32 V2 = Vertices.Emplace(CellMin + Dx * FVector3d(1, 1, 1));
					const int32 V3 = Vertices.Emplace(CellMin + Dx * FVector3d(0, 1, 1));
					Tris.Emplace(FIntVector(V0, V2, V1));
					Tris.Emplace(FIntVector(V2, V0, V3));
				}
			}
		}
	}
}

void FLevelSet::GetInteriorCells(TArray<TVec3<int32>>& InteriorCells, const FReal InteriorThreshold) const
{
	InteriorCells.Reset();

	const TVector<int32, 3> Cells = MGrid.Counts();

	for (int i = 0; i < Cells.X; ++i)
	{
		for (int j = 0; j < Cells.Y; ++j)
		{
			for (int k = 0; k < Cells.Z; ++k)
			{
				const FReal Value = MPhi(i, j, k);
				if (Value < InteriorThreshold)
				{
					InteriorCells.Emplace(i, j, k);
				}
			}
		}
	}
}

void FLevelSet::ComputeConvexity(const TArray<TVec3<int32>>& InterfaceIndices)
{
	this->bIsConvex = true;
	int32 Sign = 1;
	bool bFirst = true;
	int ZOffset = MGrid.Counts()[2];
	int YZOffset = MGrid.Counts()[1] * ZOffset;
	const int32 NumCells = MGrid.Counts().Product();
	for (const auto& Index : InterfaceIndices)
	{
		const int32 i = (Index.X * MGrid.Counts()[1] + Index.Y) * MGrid.Counts()[2] + Index.Z;
		if (MPhi[i] > 0)
		{
			continue;
		}
		int32 LocalSign;
		FReal PhiX = (MPhi[MAX_CLAMP(i + YZOffset, NumCells, i)] - MPhi[MIN_CLAMP(i - YZOffset, 0, i)]) / (2 * MGrid.Dx()[0]);
		FReal PhiXX = (MPhi[MIN_CLAMP(i - YZOffset, 0, i)] + MPhi[MAX_CLAMP(i + YZOffset, NumCells, i)] - 2 * MPhi[i]) / (MGrid.Dx()[0] * MGrid.Dx()[0]);
		FReal PhiY = (MPhi[MAX_CLAMP(i + ZOffset, NumCells, i)] - MPhi[MIN_CLAMP(i - ZOffset, 0, i)]) / (2 * MGrid.Dx()[1]);
		FReal PhiYY = (MPhi[MIN_CLAMP(i - ZOffset, 0, i)] + MPhi[MAX_CLAMP(i + ZOffset, NumCells, i)] - 2 * MPhi[i]) / (MGrid.Dx()[1] * MGrid.Dx()[1]);
		FReal PhiZ = (MPhi[MAX_CLAMP(i + 1, NumCells, i)] - MPhi[MIN_CLAMP(i - 1, 0, i)]) / (2 * MGrid.Dx()[2]);
		FReal PhiZZ = (MPhi[MIN_CLAMP(i - 1, 0, i)] + MPhi[MAX_CLAMP(i + 1, NumCells, i)] - 2 * MPhi[i]) / (MGrid.Dx()[2] * MGrid.Dx()[2]);
		FReal PhiXY = (MPhi[MAX_CLAMP(i + YZOffset + ZOffset, NumCells, i)] + MPhi[MIN_CLAMP(i - YZOffset - ZOffset, 0, i)] - MPhi[RANGE_CLAMP(i - YZOffset + ZOffset, NumCells, i)] - MPhi[RANGE_CLAMP(i + YZOffset - ZOffset, NumCells, i)]) / (4 * MGrid.Dx()[0] * MGrid.Dx()[1]);
		FReal PhiXZ = (MPhi[MAX_CLAMP(i + YZOffset + 1, NumCells, i)] + MPhi[MIN_CLAMP(i - YZOffset - 1, 0, i)] - MPhi[RANGE_CLAMP(i - YZOffset + 1, NumCells, i)] - MPhi[RANGE_CLAMP(i + YZOffset - 1, NumCells, i)]) / (4 * MGrid.Dx()[0] * MGrid.Dx()[2]);
		FReal PhiYZ = (MPhi[MAX_CLAMP(i + ZOffset + 1, NumCells, i)] + MPhi[MIN_CLAMP(i - ZOffset - 1, 0, i)] - MPhi[RANGE_CLAMP(i - ZOffset + 1, NumCells, i)] - MPhi[RANGE_CLAMP(i + ZOffset - 1, NumCells, i)]) / (4 * MGrid.Dx()[1] * MGrid.Dx()[2]);

		FReal Denom = sqrt(PhiX * PhiX + PhiY * PhiY + PhiZ * PhiZ);
		if (Denom > UE_SMALL_NUMBER)
		{
			FReal curvature = -(PhiX * PhiX * PhiYY - 2 * PhiX * PhiY * PhiXY + PhiY * PhiY * PhiXX + PhiX * PhiX * PhiZZ - 2 * PhiX * PhiZ * PhiXZ + PhiZ * PhiZ * PhiXX + PhiY * PhiY * PhiZZ - 2 * PhiY * PhiZ * PhiYZ + PhiZ * PhiZ * PhiYY) / (Denom * Denom * Denom);
			LocalSign = curvature > UE_KINDA_SMALL_NUMBER ? 1 : (curvature < -UE_KINDA_SMALL_NUMBER ? -1 : 0);
			if (bFirst)
			{
				bFirst = false;
				Sign = LocalSign;
			}
			else
			{
				if (LocalSign != 0 && Sign != LocalSign)
				{
					this->bIsConvex = false;
					return;
				}
			}
		}
	}
}

bool FLevelSet::ComputeDistancesNearZeroIsocontour(FErrorReporter& ErrorReporter, const FParticles& InParticles, const TArray<FVec3>& Normals, const FTriangleMesh& Mesh, TArrayND<bool, 3>& BlockedFaceX, TArrayND<bool, 3>& BlockedFaceY, TArrayND<bool, 3>& BlockedFaceZ, TArray<TVec3<int32>>& InterfaceIndices)
{
	MPhi.Fill(FLT_MAX);

	BlockedFaceX.Fill(false);
	BlockedFaceY.Fill(false);
	BlockedFaceZ.Fill(false);
	const TArray<TVec3<int32>>& Elements = Mesh.GetSurfaceElements();
	if (Elements.Num() > 0)
	{
		MOriginalLocalBoundingBox = FAABB3(InParticles.GetX(Elements[0][0]), InParticles.GetX(Elements[0][0]));
	}
	else
	{
		//just use bounds of grid. This should not happen
		//check(false);
		MOriginalLocalBoundingBox = MLocalBoundingBox;
	}
	for (int32 Index = 0; Index < Elements.Num(); ++Index)
	{
		const auto& Element = Elements[Index];
		TPlane<FReal, 3> TrianglePlane(InParticles.GetX(Element[0]), Normals[Index]);
		FAABB3 TriangleBounds(InParticles.GetX(Element[0]), InParticles.GetX(Element[0]));
		TriangleBounds.GrowToInclude(InParticles.GetX(Element[1]));
		TriangleBounds.GrowToInclude(InParticles.GetX(Element[2]));
		MOriginalLocalBoundingBox.GrowToInclude(TriangleBounds); //also save the original bounding box

		TVec3<int32> StartIndex = MGrid.Cell(TriangleBounds.Min() - FVec3((0.5f + UE_KINDA_SMALL_NUMBER) * MGrid.Dx()));
		TVec3<int32> EndIndex = MGrid.Cell(TriangleBounds.Max() + FVec3((0.5f + UE_KINDA_SMALL_NUMBER) * MGrid.Dx()));
		for (int32 i = StartIndex[0]; i <= EndIndex[0]; ++i)
		{
			for (int32 j = StartIndex[1]; j <= EndIndex[1]; ++j)
			{
				for (int32 k = StartIndex[2]; k <= EndIndex[2]; ++k)
				{
					const TVec3<int32> CellIndex(i, j, k);
					const FVec3 Center = MGrid.Location(CellIndex);
					const FVec3 Point = FindClosestPointOnTriangle(TrianglePlane, InParticles.GetX(Element[0]), InParticles.GetX(Element[1]), InParticles.GetX(Element[2]), Center);

					FReal NewPhi = (Point - Center).Size();
					if (NewPhi < MPhi(CellIndex))
					{
						MPhi(CellIndex) = NewPhi;
						InterfaceIndices.AddUnique(CellIndex);
					}
				}
			}
		}
		for (int32 i = StartIndex[0] + 1; i <= EndIndex[0]; ++i)
		{
			for (int32 j = StartIndex[1] + 1; j <= EndIndex[1]; ++j)
			{
				for (int32 k = StartIndex[2] + 1; k <= EndIndex[2]; ++k)
				{
					const TVec3<int32> CellIndex(i, j, k);
					if (!BlockedFaceX(CellIndex) && IsIntersectingWithTriangle(InParticles, Element, TrianglePlane, CellIndex, TVec3<int32>(i - 1, j, k)))
					{
						BlockedFaceX(CellIndex) = true;
					}
					if (!BlockedFaceY(CellIndex) && IsIntersectingWithTriangle(InParticles, Element, TrianglePlane, CellIndex, TVec3<int32>(i, j - 1, k)))
					{
						BlockedFaceY(CellIndex) = true;
					}
					if (!BlockedFaceZ(CellIndex) && IsIntersectingWithTriangle(InParticles, Element, TrianglePlane, CellIndex, TVec3<int32>(i, j, k - 1)))
					{
						BlockedFaceZ(CellIndex) = true;
					}
				}
			}
		}
	}

	// Pad resulting bounds to compensate for potentially lost volume from vertex collapsing.
	static const FVector LevelSetBoundsPadding(1.02f);
	MOriginalLocalBoundingBox.Scale(LevelSetBoundsPadding);

	return true;
}

void FLevelSet::ComputeDistancesNearZeroIsocontour(const FImplicitObject& Object, const TArrayND<FReal, 3>& ObjectPhi, TArray<TVec3<int32>>& InterfaceIndices)
{
	MPhi.Fill(FLT_MAX);
	const auto& Counts = MGrid.Counts();
	for (int32 i = 0; i < Counts[0]; ++i)
	{
		for (int32 j = 0; j < Counts[1]; ++j)
		{
			for (int32 k = 0; k < Counts[2]; ++k)
			{
				bool bBoundaryCell = false;
				const TVec3<int32> CellIndex(i, j, k);
				const TVec3<int32> CellIndexXM1(i - 1, j, k);
				const TVec3<int32> CellIndexXP1(i + 1, j, k);
				const TVec3<int32> CellIndexYM1(i, j - 1, k);
				const TVec3<int32> CellIndexYP1(i, j + 1, k);
				const TVec3<int32> CellIndexZM1(i, j, k - 1);
				const TVec3<int32> CellIndexZP1(i, j, k + 1);
				if (i > 0 && FMath::Sign(ObjectPhi(CellIndex)) != FMath::Sign(ObjectPhi(CellIndexXM1)))
				{
					bBoundaryCell = true;
				}
				if (i < (Counts[0] - 1) && FMath::Sign(ObjectPhi(CellIndex)) != FMath::Sign(ObjectPhi(CellIndexXP1)))
				{
					bBoundaryCell = true;
				}
				if (j > 0 && FMath::Sign(ObjectPhi(CellIndex)) != FMath::Sign(ObjectPhi(CellIndexYM1)))
				{
					bBoundaryCell = true;
				}
				if (j < (Counts[1] - 1) && FMath::Sign(ObjectPhi(CellIndex)) != FMath::Sign(ObjectPhi(CellIndexYP1)))
				{
					bBoundaryCell = true;
				}
				if (k > 0 && FMath::Sign(ObjectPhi(CellIndex)) != FMath::Sign(ObjectPhi(CellIndexZM1)))
				{
					bBoundaryCell = true;
				}
				if (k < (Counts[2] - 1) && FMath::Sign(ObjectPhi(CellIndex)) != FMath::Sign(ObjectPhi(CellIndexZP1)))
				{
					bBoundaryCell = true;
				}
				if (bBoundaryCell)
				{
					MPhi(CellIndex) = FMath::Abs(ObjectPhi(CellIndex));
					InterfaceIndices.Add(CellIndex);
				}
			}
		}
	}
}

void FLevelSet::CorrectSign(const TArrayND<bool, 3>& BlockedFaceX, const TArrayND<bool, 3>& BlockedFaceY, const TArrayND<bool, 3>& BlockedFaceZ, TArray<TVec3<int32>>& InterfaceIndices)
{
	int32 NextColor = -1;
	TArrayND<int32, 3> Color(MGrid);
	Color.Fill(-1);
	const auto& Counts = MGrid.Counts();
	for (int32 i = 0; i < Counts[0]; ++i)
	{
		for (int32 j = 0; j < Counts[1]; ++j)
		{
			for (int32 k = 0; k < Counts[2]; ++k)
			{
				//if we have any isolated holes or single cells near the border mark them with a color
				const TVec3<int32> CellIndex(i, j, k);
				if ((i == 0 || BlockedFaceX(CellIndex)) && (i == (Counts[0] - 1) || BlockedFaceX(TVec3<int32>(i + 1, j, k))) &&
				    (j == 0 || BlockedFaceY(CellIndex)) && (j == (Counts[1] - 1) || BlockedFaceY(TVec3<int32>(i, j + 1, k))) &&
				    (k == 0 || BlockedFaceZ(CellIndex)) && (k == (Counts[2] - 1) || BlockedFaceZ(TVec3<int32>(i, j, k + 1))))
				{
					Color(CellIndex) = ++NextColor;
				}
			}
		}
	}
	FloodFill(BlockedFaceX, BlockedFaceY, BlockedFaceZ, Color, NextColor);
	TSet<int32> BoundaryColors;
	TArray<int32> ColorIsInside;
	ColorIsInside.SetNum(NextColor + 1);
	for (int32 i = 0; i <= NextColor; ++i)
	{
		ColorIsInside[i] = -1;
	}
	for (int32 j = 0; j < Counts[1]; ++j)
	{
		for (int32 k = 0; k < Counts[2]; ++k)
		{
			int32 LColor = Color(TVec3<int32>(0, j, k));
			int32 RColor = Color(TVec3<int32>(Counts[0] - 1, j, k));
			ColorIsInside[LColor] = 0;
			ColorIsInside[RColor] = 0;
			if (!BoundaryColors.Contains(LColor))
			{
				BoundaryColors.Add(LColor);
			}
			if (!BoundaryColors.Contains(RColor))
			{
				BoundaryColors.Add(RColor);
			}
		}
	}
	for (int32 i = 0; i < Counts[0]; ++i)
	{
		for (int32 k = 0; k < Counts[2]; ++k)
		{
			int32 LColor = Color(TVec3<int32>(i, 0, k));
			int32 RColor = Color(TVec3<int32>(i, Counts[1] - 1, k));
			ColorIsInside[LColor] = 0;
			ColorIsInside[RColor] = 0;
			if (!BoundaryColors.Contains(LColor))
			{
				BoundaryColors.Add(LColor);
			}
			if (!BoundaryColors.Contains(RColor))
			{
				BoundaryColors.Add(RColor);
			}
		}
	}
	for (int32 i = 0; i < Counts[0]; ++i)
	{
		for (int32 j = 0; j < Counts[1]; ++j)
		{
			int32 LColor = Color(TVec3<int32>(i, j, 0));
			int32 RColor = Color(TVec3<int32>(i, j, Counts[2] - 1));
			ColorIsInside[LColor] = 0;
			ColorIsInside[RColor] = 0;
			if (!BoundaryColors.Contains(LColor))
			{
				BoundaryColors.Add(LColor);
			}
			if (!BoundaryColors.Contains(RColor))
			{
				BoundaryColors.Add(RColor);
			}
		}
	}
#if 0
	TMap<int32, TSet<int32>> ColorAdjacency;
	for (int32 i = 0; i <= NextColor; ++i)
	{
		ColorAdjacency.Add(i, TSet<int32>());
	}
	for (int32 i = 0; i < Counts[0]; ++i)
	{
		for (int32 j = 0; j < Counts[1]; ++j)
		{
			for (int32 k = 0; k < Counts[2]; ++k)
			{
				const auto CellIndex = TVec3<int32>(i, j, k);
				int32 SourceColor = Color(CellIndex);
				const auto CellIndexXP1 = CellIndex + TVec3<int32>::AxisVector(0);
				const auto CellIndexXM1 = CellIndex - TVec3<int32>::AxisVector(0);
				const auto CellIndexYP1 = CellIndex + TVec3<int32>::AxisVector(1);
				const auto CellIndexYM1 = CellIndex - TVec3<int32>::AxisVector(1);
				const auto CellIndexZP1 = CellIndex + TVec3<int32>::AxisVector(2);
				const auto CellIndexZM1 = CellIndex - TVec3<int32>::AxisVector(2);
				const int32 ColorXP1 = CellIndexXP1[0] < MGrid.Counts()[0] ? Color(CellIndexXP1) : -1;
				const int32 ColorXM1 = CellIndexXM1[0] >= 0 ? Color(CellIndexXM1) : -1;
				const int32 ColorYP1 = CellIndexYP1[1] < MGrid.Counts()[1] ? Color(CellIndexYP1) : -1;
				const int32 ColorYM1 = CellIndexYM1[1] >= 0 ? Color(CellIndexYM1) : -1;
				const int32 ColorZP1 = CellIndexZP1[2] < MGrid.Counts()[2] ? Color(CellIndexZP1) : -1;
				const int32 ColorZM1 = CellIndexZM1[2] >= 0 ? Color(CellIndexZM1) : -1;
				if (CellIndexZP1[2] < MGrid.Counts()[2] && SourceColor != ColorZP1 && !ColorAdjacency[SourceColor].Contains(ColorZP1))
				{
					ColorAdjacency[SourceColor].Add(ColorZP1);
				}
				if (CellIndexZM1[2] >= 0 && SourceColor != ColorZM1 && !ColorAdjacency[SourceColor].Contains(ColorZM1))
				{
					ColorAdjacency[SourceColor].Add(ColorZM1);
				}
				if (CellIndexYP1[1] < MGrid.Counts()[1] && SourceColor != ColorYP1 && !ColorAdjacency[SourceColor].Contains(ColorYP1))
				{
					ColorAdjacency[SourceColor].Add(ColorYP1);
				}
				if (CellIndexYM1[1] >= 0 && SourceColor != ColorYM1 && !ColorAdjacency[SourceColor].Contains(ColorYM1))
				{
					ColorAdjacency[SourceColor].Add(ColorYM1);
				}
				if (CellIndexXP1[0] < MGrid.Counts()[0] && SourceColor != ColorXP1 && !ColorAdjacency[SourceColor].Contains(ColorXP1))
				{
					ColorAdjacency[SourceColor].Add(ColorXP1);
				}
				if (CellIndexXM1[0] >= 0 && SourceColor != ColorXM1 && !ColorAdjacency[SourceColor].Contains(ColorXM1))
				{
					ColorAdjacency[SourceColor].Add(ColorXM1);
				}
			}
		}
	}
	TQueue<int32> ProcessedColors;
	for (const int32 BoundaryColor : BoundaryColors)
	{
		ProcessedColors.Enqueue(BoundaryColor);
	}
	while (!ProcessedColors.IsEmpty())
	{
		int32 CurrentColor;
		ProcessedColors.Dequeue(CurrentColor);
		for (const int32 AdjacentColor : ColorAdjacency[CurrentColor])
		{
			if (ColorIsInside[AdjacentColor] < 0)
			{
				ColorIsInside[AdjacentColor] = !ColorIsInside[CurrentColor];
				ProcessedColors.Enqueue(AdjacentColor);
			}
			else
			{
				check(ColorIsInside[AdjacentColor] == !ColorIsInside[CurrentColor]);
			}
		}
	}
#endif
	for (int32 i = 0; i < Counts[0]; ++i)
	{
		for (int32 j = 0; j < Counts[1]; ++j)
		{
			for (int32 k = 0; k < Counts[2]; ++k)
			{
				const TVec3<int32> CellIndex(i, j, k);
				if (ColorIsInside[Color(CellIndex)])
				{
					MPhi(CellIndex) *= -1;
				}
			}
		}
	}

	//remove internal cells from interface list
	for (int32 i = InterfaceIndices.Num() - 1; i >= 0; --i)
	{
		const TVec3<int32>& CellIndex = InterfaceIndices[i];
		if (!ColorIsInside[Color(CellIndex)])
		{
			continue; //already an outside color
		}

		bool bInside = true;
		for (int32 Axis = 0; Axis < 3; ++Axis)
		{
			//if any neighbors are outside this is a real interface cell
			const TVec3<int32> IndexP1 = CellIndex + TVec3<int32>::AxisVector(Axis);

			if (IndexP1[Axis] >= MGrid.Counts()[Axis] || !ColorIsInside[Color(IndexP1)])
			{
				bInside = false;
				break;
			}

			const TVec3<int32> IndexM1 = CellIndex - TVec3<int32>::AxisVector(Axis);
			if (IndexM1[Axis] < 0 || !ColorIsInside[Color(IndexM1)])
			{
				bInside = false;
				break;
			}
		}

		if (bInside)
		{
			//fully internal cell so remove it from interface list
			MPhi(CellIndex) = -FLT_MAX;
			InterfaceIndices.RemoveAt(i);
		}
	}
}

bool Compare(const Pair<FReal*, TVec3<int32>>& Other1, const Pair<FReal*, TVec3<int32>>& Other2)
{
	return FMath::Abs(*Other1.First) < FMath::Abs(*Other2.First);
}

void FLevelSet::FillWithFastMarchingMethod(const FReal StoppingDistance, const TArray<TVec3<int32>>& InterfaceIndices)
{
	TArrayND<bool, 3> Done(MGrid), InHeap(MGrid);
	Done.Fill(false);
	InHeap.Fill(false);
	TArray<Pair<FReal*, TVec3<int32>>> Heap;
	// TODO(mlentine): Update phi for these cells
	for (const auto& CellIndex : InterfaceIndices)
	{
		check(!Done(CellIndex) && !InHeap(CellIndex));
		Done(CellIndex) = true;
		Heap.Add(MakePair(&MPhi(CellIndex), CellIndex));
		InHeap(CellIndex) = true;
	}
	Heap.Heapify(Compare);
	while (Heap.Num())
	{
		Pair<FReal*, TVec3<int32>> Smallest;
		Heap.HeapPop(Smallest, Compare);
		check(InHeap(Smallest.Second));
		if (StoppingDistance != 0 && FGenericPlatformMath::Abs(*Smallest.First) > StoppingDistance)
		{
			break;
		}
		Done(Smallest.Second) = true;
		InHeap(Smallest.Second) = false;
		for (int32 Axis = 0; Axis < 3; ++Axis)
		{
			const auto IP1 = Smallest.Second + TVec3<int32>::AxisVector(Axis);
			const auto IM1 = Smallest.Second - TVec3<int32>::AxisVector(Axis);
			if (IM1[Axis] >= 0 && !Done(IM1))
			{
				MPhi(IM1) = ComputePhi(Done, IM1);
				if (!InHeap(IM1))
				{
					Heap.Add(MakePair(&MPhi(IM1), IM1));
					InHeap(IM1) = true;
				}
			}
			if (IP1[Axis] < MGrid.Counts()[Axis] && !Done(IP1))
			{
				MPhi(IP1) = ComputePhi(Done, IP1);
				if (!InHeap(IP1))
				{
					Heap.Add(MakePair(&MPhi(IP1), IP1));
					InHeap(IP1) = true;
				}
			}
		}
		Heap.Heapify(Compare);
	}
}

FReal SolveQuadraticEquation(const FReal Phi, const FReal PhiX, const FReal PhiY, const FReal Dx, const FReal Dy)
{
	check(FMath::Sign(PhiX) == FMath::Sign(PhiY) || FMath::Sign(PhiX) == 0 || FMath::Sign(PhiY) == 0);
	FReal Sign = Phi > 0 ? (FReal)1.0 : (FReal)-1.0;
	if (FMath::Abs(PhiX) >= (FMath::Abs(PhiY) + Dy))
	{
		return PhiY + Sign * Dy;
	}
	if (FMath::Abs(PhiY) >= (FMath::Abs(PhiX) + Dx))
	{
		return PhiX + Sign * Dx;
	}
	FReal Dx2 = Dx * Dx;
	FReal Dy2 = Dy * Dy;
	FReal Diff = PhiX - PhiY;
	FReal Diff2 = Diff * Diff;
	return (Dy2 * PhiX + Dx2 * PhiY + Sign * Dx * Dy * FMath::Sqrt(Dx2 + Dy2 - Diff2)) / (Dx2 + Dy2);
}

FReal FLevelSet::ComputePhi(const TArrayND<bool, 3>& Done, const TVec3<int32>& CellIndex)
{
	int32 NumberOfAxes = 0;
	FVec3 NeighborPhi;
	FVec3 Dx;
	for (int32 Axis = 0; Axis < 3; ++Axis)
	{
		const auto IP1 = CellIndex + TVec3<int32>::AxisVector(Axis);
		const auto IM1 = CellIndex - TVec3<int32>::AxisVector(Axis);
		if (IM1[Axis] < 0 || !Done(IM1)) // IM1 not valid
		{
			if (IP1[Axis] < MGrid.Counts()[Axis] && Done(IP1)) // IP1 is valid
			{
				Dx[NumberOfAxes] = MGrid.Dx()[Axis];
				NeighborPhi[NumberOfAxes++] = MPhi(IP1);
			}
		}
		else if (IP1[Axis] >= MGrid.Counts()[Axis] || !Done(IP1))
		{
			Dx[NumberOfAxes] = MGrid.Dx()[Axis];
			NeighborPhi[NumberOfAxes++] = MPhi(IM1);
		}
		else
		{
			Dx[NumberOfAxes] = MGrid.Dx()[Axis];
			if (FMath::Abs(MPhi(IP1)) < FMath::Abs(MPhi(IM1)))
			{
				NeighborPhi[NumberOfAxes++] = MPhi(IP1);
			}
			else
			{
				NeighborPhi[NumberOfAxes++] = MPhi(IM1);
			}
		}
	}
	if (NumberOfAxes == 1)
	{
		FReal Sign = MPhi(CellIndex) > 0 ? (FReal)1.0 : (FReal)-1.0;
		FReal NewPhi = FGenericPlatformMath::Abs(NeighborPhi[0]) + Dx[0];
		check(NewPhi <= FGenericPlatformMath::Abs(MPhi(CellIndex)));
		return Sign * NewPhi;
	}
	FReal QuadraticXY = SolveQuadraticEquation(MPhi(CellIndex), NeighborPhi[0], NeighborPhi[1], Dx[0], Dx[1]);
	if (NumberOfAxes == 2 || FMath::Abs(NeighborPhi[2]) > FMath::Abs(QuadraticXY))
	{
		return QuadraticXY;
	}
	FReal QuadraticXZ = SolveQuadraticEquation(MPhi(CellIndex), NeighborPhi[0], NeighborPhi[2], Dx[0], Dx[2]);
	if (FMath::Abs(NeighborPhi[1]) > FMath::Abs(QuadraticXZ))
	{
		return QuadraticXZ;
	}
	FReal QuadraticYZ = SolveQuadraticEquation(MPhi(CellIndex), NeighborPhi[1], NeighborPhi[2], Dx[1], Dx[2]);
	if (FMath::Abs(NeighborPhi[0]) > FMath::Abs(QuadraticYZ))
	{
		return QuadraticYZ;
	}
	// Cubic
	FReal Sign = MPhi(CellIndex) > 0 ? (FReal)1.0 : (FReal)-1;
	FReal Dx2 = Dx[0] * Dx[0];
	FReal Dy2 = Dx[1] * Dx[1];
	FReal Dz2 = Dx[2] * Dx[2];
	FReal Dx2Dy2 = Dx2 * Dy2;
	FReal Dx2Dz2 = Dx2 * Dz2;
	FReal Dy2Dz2 = Dy2 * Dz2;
	FReal XmY = NeighborPhi[0] - NeighborPhi[1];
	FReal XmZ = NeighborPhi[0] - NeighborPhi[2];
	FReal YmZ = NeighborPhi[1] - NeighborPhi[2];
	FReal XmY2 = XmY * XmY;
	FReal XmZ2 = XmZ * XmZ;
	FReal YmZ2 = YmZ * YmZ;
	FReal UnderRoot = Dx2Dy2 + Dx2Dz2 + Dy2Dz2 - Dx2 * YmZ2 - Dy2 * XmZ2 - Dz2 * XmY2;
	if (UnderRoot < 0)
	{
		UnderRoot = 0;
	}
	return (Dy2Dz2 * NeighborPhi[0] + Dx2Dz2 * NeighborPhi[1] + Dx2Dy2 * NeighborPhi[2] + Sign * Dx.Product() * FMath::Sqrt(UnderRoot)) / (Dx2Dy2 + Dx2Dz2 + Dy2Dz2);
}

void FLevelSet::FloodFill(const TArrayND<bool, 3>& BlockedFaceX, const TArrayND<bool, 3>& BlockedFaceY, const TArrayND<bool, 3>& BlockedFaceZ, TArrayND<int32, 3>& Color, int32& NextColor)
{
	const auto& Counts = MGrid.Counts();
	for (int32 i = 0; i < Counts[0]; ++i)
	{
		for (int32 j = 0; j < Counts[1]; ++j)
		{
			for (int32 k = 0; k < Counts[2]; ++k)
			{
				const TVec3<int32> CellIndex(i, j, k);
				if (Color(CellIndex) == -1)
				{
					FloodFillFromCell(CellIndex, ++NextColor, BlockedFaceX, BlockedFaceY, BlockedFaceZ, Color);
					check(Color(CellIndex) != -1);
				}
			}
		}
	}
}

void FLevelSet::FloodFillFromCell(const TVec3<int32> RootCellIndex, const int32 NextColor, const TArrayND<bool, 3>& BlockedFaceX, const TArrayND<bool, 3>& BlockedFaceY, const TArrayND<bool, 3>& BlockedFaceZ, TArrayND<int32, 3>& Color)
{
	TArray<TVec3<int32>> Queue;
	Queue.Add(RootCellIndex);
	while (Queue.Num())
	{
		TVec3<int32> CellIndex = Queue.Pop();
		if (Color(CellIndex) == NextColor)
		{
			continue;
		}

		ensure(Color(CellIndex) == -1);
		Color(CellIndex) = NextColor;
		const auto CellIndexXP1 = CellIndex + TVec3<int32>::AxisVector(0);
		const auto CellIndexXM1 = CellIndex - TVec3<int32>::AxisVector(0);
		const auto CellIndexYP1 = CellIndex + TVec3<int32>::AxisVector(1);
		const auto CellIndexYM1 = CellIndex - TVec3<int32>::AxisVector(1);
		const auto CellIndexZP1 = CellIndex + TVec3<int32>::AxisVector(2);
		const auto CellIndexZM1 = CellIndex - TVec3<int32>::AxisVector(2);
		if (CellIndexZP1[2] < MGrid.Counts()[2] && !BlockedFaceZ(CellIndexZP1) && Color(CellIndexZP1) != NextColor)
		{
			Queue.Add(CellIndexZP1);
		}
		if (!BlockedFaceZ(CellIndex) && CellIndexZM1[2] >= 0 && Color(CellIndexZM1) != NextColor)
		{
			Queue.Add(CellIndexZM1);
		}
		if (CellIndexYP1[1] < MGrid.Counts()[1] && !BlockedFaceY(CellIndexYP1) && Color(CellIndexYP1) != NextColor)
		{
			Queue.Add(CellIndexYP1);
		}
		if (!BlockedFaceY(CellIndex) && CellIndexYM1[1] >= 0 && Color(CellIndexYM1) != NextColor)
		{
			Queue.Add(CellIndexYM1);
		}
		if (CellIndexXP1[0] < MGrid.Counts()[0] && !BlockedFaceX(CellIndexXP1) && Color(CellIndexXP1) != NextColor)
		{
			Queue.Add(CellIndexXP1);
		}

		if (!BlockedFaceX(CellIndex) && CellIndexXM1[0] >= 0 && Color(CellIndexXM1) != NextColor)
		{
			Queue.Add(CellIndexXM1);
		}
	}
}

bool FLevelSet::IsIntersectingWithTriangle(const FParticles& Particles, const TVec3<int32>& Element, const TPlane<FReal, 3>& TrianglePlane, const TVec3<int32>& CellIndex, const TVec3<int32>& PrevCellIndex)
{
	auto Intersection = TrianglePlane.FindClosestIntersection(MGrid.Location(CellIndex), MGrid.Location(PrevCellIndex), 0);
	if (Intersection.Second)
	{
		const FReal Epsilon = (FReal)1e-1; //todo(ocohen): fattening triangle up is relative to triangle size. Do we care about very large triangles?
		const FVec2 Bary = ComputeBarycentricInPlane(Particles.GetX(Element[0]), Particles.GetX(Element[1]), Particles.GetX(Element[2]), Intersection.First);

		if (Bary.X >= -Epsilon && Bary.Y >= -Epsilon && (Bary.Y + Bary.X) <= 1 + Epsilon)
		{
			return true;
		}
	}
	return false;
}

void FLevelSet::ComputeNormals()
{
	const auto& Counts = MGrid.Counts();
	for (int32 i = 0; i < Counts[0]; ++i)
	{
		for (int32 j = 0; j < Counts[1]; ++j)
		{
			for (int32 k = 0; k < Counts[2]; ++k)
			{
				const TVec3<int32> CellIndex(i, j, k);
				const auto Dx = MGrid.Dx();
				FVec3 X = MGrid.Location(CellIndex);
				MNormals(CellIndex) = FVec3(
				    (SignedDistance(X + FVec3::AxisVector(0) * Dx[0]) - SignedDistance(X - FVec3::AxisVector(0) * Dx[0])) / (2 * Dx[0]),
				    (SignedDistance(X + FVec3::AxisVector(1) * Dx[1]) - SignedDistance(X - FVec3::AxisVector(1) * Dx[1])) / (2 * Dx[1]),
				    (SignedDistance(X + FVec3::AxisVector(2) * Dx[2]) - SignedDistance(X - FVec3::AxisVector(2) * Dx[2])) / (2 * Dx[2]));
				FReal Size = MNormals(CellIndex).Size();
				if (Size > UE_SMALL_NUMBER)
				{
					MNormals(CellIndex) /= Size;
				}
				else
				{
					MNormals(CellIndex) = FVec3(0);
					MNormals(CellIndex).X = (FReal)1;
				}
			}
		}
	}
}

// @todo(mlentine): This is super expensive but until we know it is working it's better to keep it outside of main level set generation
void FLevelSet::ComputeNormals(const FParticles& InParticles, const FTriangleMesh& Mesh, const TArray<TVec3<int32>>& InterfaceIndices)
{
	ComputeNormals();
	const TArray<FVec3> Normals = Mesh.GetFaceNormals(InParticles);
	if (Normals.Num() == 0)
	{
		return;
	}
	TArrayND<bool, 3> Done(MGrid), InHeap(MGrid);
	Done.Fill(false);
	InHeap.Fill(false);
	TArrayND<FReal, 3> LocalPhi(MGrid);
	LocalPhi.Fill(FLT_MAX);
	TArray<Pair<FReal*, TVec3<int32>>> Heap;
	TSet<TVec3<int32>> InterfaceSet;
	for (const TVec3<int32>& Index : InterfaceIndices)
	{
		InterfaceSet.Add(Index);
	}

	const TArray<TVec3<int32>>& Elements = Mesh.GetSurfaceElements();
	if (Elements.Num() > 0)
	{
		MOriginalLocalBoundingBox = FAABB3(InParticles.GetX(Elements[0][0]), InParticles.GetX(Elements[0][0]));
	}
	else
	{
		MOriginalLocalBoundingBox = MLocalBoundingBox;
	}
	for (int32 Index = 0; Index < Elements.Num(); ++Index)
	{
		const auto& Element = Elements[Index];
		TPlane<FReal, 3> TrianglePlane(InParticles.GetX(Element[0]), Normals[Index]);
		FAABB3 TriangleBounds(InParticles.GetX(Element[0]), InParticles.GetX(Element[0]));
		TriangleBounds.GrowToInclude(InParticles.GetX(Element[1]));
		TriangleBounds.GrowToInclude(InParticles.GetX(Element[2]));
		MOriginalLocalBoundingBox.GrowToInclude(TriangleBounds); //also save the original bounding box

		TVec3<int32> StartIndex = MGrid.Cell(TriangleBounds.Min() - FVec3((0.5f + UE_KINDA_SMALL_NUMBER) * MGrid.Dx()));
		TVec3<int32> EndIndex = MGrid.Cell(TriangleBounds.Max() + FVec3((0.5f + UE_KINDA_SMALL_NUMBER) * MGrid.Dx()));
		for (int32 i = StartIndex[0]; i <= EndIndex[0]; ++i)
		{
			for (int32 j = StartIndex[1]; j <= EndIndex[1]; ++j)
			{
				for (int32 k = StartIndex[2]; k <= EndIndex[2]; ++k)
				{
					const TVec3<int32> CellIndex(i, j, k);
					if (!InterfaceSet.Contains(CellIndex))
					{
						continue;
					}
					const FVec3 Center = MGrid.Location(CellIndex);
					const FVec3 Point = FindClosestPointOnTriangle(TrianglePlane, InParticles.GetX(Element[0]), InParticles.GetX(Element[1]), InParticles.GetX(Element[2]), Center);

					FReal NewPhi = (Point - Center).Size();
					if (NewPhi < LocalPhi(CellIndex))
					{
						if (FVec3::DotProduct(MNormals(CellIndex), Normals[Index]) >= 0)
						{
							MNormals(CellIndex) = Normals[Index];
						}
						else
						{
							MNormals(CellIndex) = -Normals[Index];
						}
						if (!InHeap(CellIndex))
						{
							Done(CellIndex) = true;
							Heap.Add(MakePair(&LocalPhi(CellIndex), CellIndex));
							InHeap(CellIndex) = true;
						}
					}
				}
			}
		}
	}

	Heap.Heapify(Compare);
	while (Heap.Num())
	{
		Pair<FReal*, TVec3<int32>> Smallest;
		Heap.HeapPop(Smallest, Compare);
		check(InHeap(Smallest.Second));
		Done(Smallest.Second) = true;
		InHeap(Smallest.Second) = false;
		for (int32 Axis = 0; Axis < 3; ++Axis)
		{
			const auto IP1 = Smallest.Second + TVec3<int32>::AxisVector(Axis);
			const auto IM1 = Smallest.Second - TVec3<int32>::AxisVector(Axis);
			if (IM1[Axis] >= 0 && !Done(IM1))
			{
				if (LocalPhi(IM1) > (LocalPhi(Smallest.Second) + MGrid.Dx()[Axis]))
				{
					LocalPhi(IM1) = LocalPhi(Smallest.Second) + MGrid.Dx()[Axis];
					MNormals(IM1) = MNormals(Smallest.Second);
				}
				if (!InHeap(IM1))
				{
					Heap.Add(MakePair(&LocalPhi(IM1), IM1));
					InHeap(IM1) = true;
				}
			}
			if (IP1[Axis] < MGrid.Counts()[Axis] && !Done(IP1))
			{
				if (LocalPhi(IP1) > (LocalPhi(Smallest.Second) + MGrid.Dx()[Axis]))
				{
					LocalPhi(IP1) = LocalPhi(Smallest.Second) + MGrid.Dx()[Axis];
					MNormals(IP1) = MNormals(Smallest.Second);
				}
				if (!InHeap(IP1))
				{
					Heap.Add(MakePair(&LocalPhi(IP1), IP1));
					InHeap(IP1) = true;
				}
			}
		}
		Heap.Heapify(Compare);
	}
}

#if COMPILE_WITHOUT_UNREAL_SUPPORT
void FLevelSet::Write(std::ostream& Stream) const
{
	MGrid.Write(Stream);
	MPhi.Write(Stream);
	Stream.write(reinterpret_cast<const char*>(&MBandWidth), sizeof(MBandWidth));
}
#endif

FReal FLevelSet::SignedDistance(const FVec3& x) const
{
	FVec3 Location = MGrid.ClampMinusHalf(x);
	FReal SizeSquared = (Location - x).SizeSquared();
	FReal Phi = MGrid.LinearlyInterpolate(MPhi, Location);
	return SizeSquared > 0 ? (sqrt(SizeSquared) + Phi) : Phi;
}

FReal FLevelSet::PhiWithNormal(const FVec3& x, FVec3& Normal) const
{
	FVec3 Location = MGrid.ClampMinusHalf(x);
	FReal SizeSquared = (Location - x).SizeSquared();
	if (SizeSquared > 0)
	{
		MLocalBoundingBox.PhiWithNormal(Location, Normal);
	}
	else
	{
		Normal = MGrid.LinearlyInterpolate(MNormals, Location);
		FReal NormalMag = Normal.Size();
		if (NormalMag > UE_SMALL_NUMBER)
		{
			Normal /= NormalMag;
		}
		else
		{
			Normal = FVec3(0);
			Normal.X = (FReal)1;
		}
	}
	FReal Phi = MGrid.LinearlyInterpolate(MPhi, Location);
	return SizeSquared > 0 ? (sqrt(SizeSquared) + Phi) : Phi;
}

void GetGeomSurfaceSamples(const FSphere& InGeom, TArray<FVec3>& OutSamples)
{
	OutSamples.Reset();
	OutSamples.AddUninitialized(6);

	const FReal Radius = InGeom.GetRadiusf();

	OutSamples[0] = FVec3(Radius, 0, 0);
	OutSamples[1] = FVec3(-Radius, 0, 0);
	OutSamples[2] = FVec3(0, Radius, Radius);
	OutSamples[3] = FVec3(0, -Radius, Radius);
	OutSamples[4] = FVec3(0, -Radius, -Radius);
	OutSamples[5] = FVec3(0, Radius, -Radius);
}

void GetGeomSurfaceSamples(const TBox<FReal, 3>& InGeom, TArray<FVec3>& OutSamples)
{
	OutSamples.Reset();
	OutSamples.AddUninitialized(8);

	const FVec3& Min = InGeom.Min();
	const FVec3& Max = InGeom.Max();

	OutSamples[0] = FVec3(Min.X, Min.Y, Min.Z);
	OutSamples[1] = FVec3(Min.X, Min.Y, Max.Z);
	OutSamples[2] = FVec3(Min.X, Max.Y, Min.Z);
	OutSamples[3] = FVec3(Max.X, Min.Y, Min.Z);
	OutSamples[4] = FVec3(Max.X, Max.Y, Max.Z);
	OutSamples[5] = FVec3(Max.X, Max.Y, Min.Z);
	OutSamples[6] = FVec3(Max.X, Min.Y, Max.Z);
	OutSamples[7] = FVec3(Min.X, Max.Y, Max.Z);
}

void GetGeomSurfaceSamples(const FCapsule& InGeom, TArray<FVec3>& OutSamples)
{
	OutSamples.Reset();
	OutSamples.AddUninitialized(14);

	FReal Radius = InGeom.GetRadiusf();
	FReal HalfHeight = InGeom.GetHeightf() * 0.5f;

	OutSamples[0]	= FVec3(HalfHeight + Radius, 0, 0);
	OutSamples[1]	= FVec3(-HalfHeight - Radius, 0, 0);
	OutSamples[2]	= FVec3(HalfHeight, Radius, Radius);
	OutSamples[3]	= FVec3(HalfHeight, -Radius, Radius);
	OutSamples[4]	= FVec3(HalfHeight, -Radius, -Radius);
	OutSamples[5]	= FVec3(HalfHeight, Radius, -Radius);
	OutSamples[6]	= FVec3(0, Radius, Radius);
	OutSamples[7]	= FVec3(0, -Radius, Radius);
	OutSamples[8]	= FVec3(0, -Radius, -Radius);
	OutSamples[9]	= FVec3(0, Radius, -Radius);
	OutSamples[10]	= FVec3(-HalfHeight, Radius, Radius);
	OutSamples[11]	= FVec3(-HalfHeight, -Radius, Radius);
	OutSamples[12]	= FVec3(-HalfHeight, -Radius, -Radius);
	OutSamples[13]	= FVec3(-HalfHeight, Radius, -Radius);
}

void GetGeomSurfaceSamples(const FConvex& InGeom, TArray<FVec3>& OutSamples)
{
	// because Convex store in single precision we need to convert to FReal precision
	const TArray<FConvex::FVec3Type>& ConvexVertices = InGeom.GetVertices();
	OutSamples.Reset(ConvexVertices.Num());
	for (const FConvex::FVec3Type& Vertex : ConvexVertices)
	{
		OutSamples.Add(FVec3{ Vertex }); // conversion from single to double precision
	}
}

template<typename InnerT>
void GetGeomSurfaceSamples(const TImplicitObjectScaled<InnerT>& InScaledGeom, TArray<FVec3>& OutSamples)
{
	const InnerT* InnerObject = InScaledGeom.Object().GetReference();

	if(InnerObject)
	{
		GetGeomSurfaceSamples(*InnerObject, OutSamples);

		const FVec3 Scale = InScaledGeom.GetScale();

		for(FVec3& Sample : OutSamples)
		{
			Sample *= Scale;
		}
	}
}

template <typename QueryGeomType>
bool FLevelSet::SweepGeomImp(const QueryGeomType& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length,
	FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, const FReal Thickness, const bool bComputeMTD) const
{
	TArray<FVec3> Samples;
	GetGeomSurfaceSamples(QueryGeom, Samples);

	OutTime = TNumericLimits<FReal>::Max();
	FReal TempTime = TNumericLimits<FReal>::Max();
	FVec3 TempNormal(0);
	FVec3 TempPosition(0);
	int32 FaceIndex;

	bool bHit = false;

	for(const FVec3& Sample : Samples)
	{
		FVec3 Transformed = StartTM.TransformPosition(Sample);
		Raycast(Transformed, Dir, Length, 0, TempTime, TempPosition, TempNormal, FaceIndex);

		if(TempTime < OutTime)
		{
			OutTime = TempTime;
			OutPosition = TempPosition;
			OutNormal = TempNormal;
			bHit = true;
		}
	}

	return bHit;
}

bool FLevelSet::SweepGeom(const FSphere& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, const FReal Thickness, const bool bComputeMTD) const
{
	return SweepGeomImp(QueryGeom, StartTM, Dir, Length, OutTime, OutPosition, OutNormal, OutFaceIndex, Thickness, bComputeMTD);
}

bool FLevelSet::SweepGeom(const TBox<FReal, 3>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, const FReal Thickness, const bool bComputeMTD) const
{
	return SweepGeomImp(QueryGeom, StartTM, Dir, Length, OutTime, OutPosition, OutNormal, OutFaceIndex, Thickness, bComputeMTD);
}

bool FLevelSet::SweepGeom(const FCapsule& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, const FReal Thickness, const bool bComputeMTD) const
{
	return SweepGeomImp(QueryGeom, StartTM, Dir, Length, OutTime, OutPosition, OutNormal, OutFaceIndex, Thickness, bComputeMTD);
}

bool FLevelSet::SweepGeom(const FConvex& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, const FReal Thickness, const bool bComputeMTD) const
{
	return SweepGeomImp(QueryGeom, StartTM, Dir, Length, OutTime, OutPosition, OutNormal, OutFaceIndex, Thickness, bComputeMTD);
}

bool FLevelSet::SweepGeom(const TImplicitObjectScaled<FSphere>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, const FReal Thickness, const bool bComputeMTD) const
{
	return SweepGeomImp(QueryGeom, StartTM, Dir, Length, OutTime, OutPosition, OutNormal, OutFaceIndex, Thickness, bComputeMTD);
}

bool FLevelSet::SweepGeom(const TImplicitObjectScaled<TBox<FReal, 3>>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, const FReal Thickness, const bool bComputeMTD) const
{
	return SweepGeomImp(QueryGeom, StartTM, Dir, Length, OutTime, OutPosition, OutNormal, OutFaceIndex, Thickness, bComputeMTD);
}

bool FLevelSet::SweepGeom(const TImplicitObjectScaled<FCapsule>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, const FReal Thickness, const bool bComputeMTD) const
{
	return SweepGeomImp(QueryGeom, StartTM, Dir, Length, OutTime, OutPosition, OutNormal, OutFaceIndex, Thickness, bComputeMTD);
}

bool FLevelSet::SweepGeom(const TImplicitObjectScaled<FConvex>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, const FReal Thickness, const bool bComputeMTD) const
{
	return SweepGeomImp(QueryGeom, StartTM, Dir, Length, OutTime, OutPosition, OutNormal, OutFaceIndex, Thickness, bComputeMTD);
}

void GetGeomSurfaceSamplesExtended(const FSphere& InGeom, TArray<FVec3>& OutSamples)
{
	OutSamples = InGeom.ComputeLocalSamplePoints(NumOverlapSphereSamples);
}

void GetGeomSurfaceSamplesExtended(const TBox<FReal, 3>& InGeom, TArray<FVec3>& OutSamples)
{
	OutSamples = InGeom.ComputeLocalSamplePoints();
}

void GetGeomSurfaceSamplesExtended(const FCapsule& InGeom, TArray<FVec3>& OutSamples)
{
	OutSamples = InGeom.ComputeLocalSamplePoints(NumOverlapCapsuleSamples);
}

void GetGeomSurfaceSamplesExtended(const FConvex& InGeom, TArray<FVec3>& OutSamples)
{
	// Convex doesn't have extended samples
	GetGeomSurfaceSamples(InGeom, OutSamples);
}

template<typename InnerT>
void GetGeomSurfaceSamplesExtended(const TImplicitObjectScaled<InnerT>& InScaledGeom, TArray<FVec3>& OutSamples)
{
	const InnerT* InnerObject = InScaledGeom.Object().GetReference();

	if(InnerObject)
	{
		GetGeomSurfaceSamplesExtended(*InnerObject, OutSamples);

		const FVec3 Scale = InScaledGeom.GetScale();

		for(FVec3& Sample : OutSamples)
		{
			Sample *= Scale;
		}
	}
}

template <typename QueryGeomType>
bool FLevelSet::OverlapGeomImp(const QueryGeomType& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD) const
{
	// NOTE: This isn't a perfect overlap implementation. It takes particle samples from the query
	// geoemetry and looks for intersections, often this means that we're only detecting on the surface
	// of the query geometry.
	// #BGTODO better sampling of levelset, then invert the check for levelset point inside querygeom
	bool bResult = false;

	if(OutMTD)
	{
		OutMTD->Normal = FVec3(0);
		OutMTD->Penetration = 0;
	}

	TArray<FVec3> SamplePoints;
	FVec3 TempNormal;
	FReal TempPhi;

	// Use an extended set of points here to attempt to get a better overlap
	GetGeomSurfaceSamplesExtended(QueryGeom, SamplePoints);

	for(const FVec3& Sample : SamplePoints)
	{
		const FVec3 Transformed = QueryTM.TransformPosition(Sample);
		TempPhi = PhiWithNormal(Transformed, TempNormal);

		if(OutMTD && (-TempPhi) > OutMTD->Penetration)
		{
			OutMTD->Penetration = -TempPhi;
			OutMTD->Normal = TempNormal;
			OutMTD->Position = Transformed + OutMTD->Penetration * OutMTD->Normal;
			bResult = true;
		}
		else
		{
			if(TempPhi <= 0)
			{
				return true;
			}
		}
	}

	return bResult;
}

bool FLevelSet::OverlapGeom(const FSphere& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD) const
{
	return OverlapGeomImp(QueryGeom, QueryTM, Thickness, OutMTD);
}

bool FLevelSet::OverlapGeom(const TBox<FReal, 3>& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD) const
{
	return OverlapGeomImp(QueryGeom, QueryTM, Thickness, OutMTD);
}

bool FLevelSet::OverlapGeom(const FCapsule& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD) const
{
	return OverlapGeomImp(QueryGeom, QueryTM, Thickness, OutMTD);
}

bool FLevelSet::OverlapGeom(const FConvex& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD) const
{
	return OverlapGeomImp(QueryGeom, QueryTM, Thickness, OutMTD);
}

bool FLevelSet::OverlapGeom(const TImplicitObjectScaled<FSphere>& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD) const
{
	return OverlapGeomImp(QueryGeom, QueryTM, Thickness, OutMTD);
}

bool FLevelSet::OverlapGeom(const TImplicitObjectScaled<TBox<FReal, 3>>& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD) const
{
	return OverlapGeomImp(QueryGeom, QueryTM, Thickness, OutMTD);
}

bool FLevelSet::OverlapGeom(const TImplicitObjectScaled<FCapsule>& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD) const
{
	return OverlapGeomImp(QueryGeom, QueryTM, Thickness, OutMTD);
}

bool FLevelSet::OverlapGeom(const TImplicitObjectScaled<FConvex>& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD) const
{
	return OverlapGeomImp(QueryGeom, QueryTM, Thickness, OutMTD);
}
}

#undef MAX_CLAMP
#undef MIN_CLAMP
#undef RANGE_CLAMP
