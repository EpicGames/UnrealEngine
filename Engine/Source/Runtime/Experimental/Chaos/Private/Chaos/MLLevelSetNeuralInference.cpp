// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/MLLevelSetNeuralInference.h"

namespace Chaos
{

CHAOS_API FMLLevelSetNeuralInference::FMLLevelSetNeuralInference(TSharedPtr<UE::NNE::IModelCPU> NNEModel, TArray<TArray<int32>>& InModelWeightsShapes)
	: NNEModel(NNEModel)
	, ModelWeightsShapes(InModelWeightsShapes)
{
	ModelInstance = NNEModel->CreateModelInstanceCPU();
}

CHAOS_API FMLLevelSetNeuralInference::FMLLevelSetNeuralInference(TSharedPtr<UE::NNE::IModelCPU> NNEModel)
	: NNEModel(NNEModel)
{
	ModelInstance = NNEModel->CreateModelInstanceCPU();
}

CHAOS_API FMLLevelSetNeuralInference::FMLLevelSetNeuralInference(FMLLevelSetNeuralInference&& Other)
	: NNEModel(Other.NNEModel)
	, ModelInstance(Other.ModelInstance)
	, ModelWeightsShapes(MoveTemp(Other.ModelWeightsShapes))
{}

CHAOS_API FMLLevelSetNeuralInference::FMLLevelSetNeuralInference(const FMLLevelSetNeuralInference& Other)
	: NNEModel(Other.NNEModel)
	, ModelWeightsShapes(Other.ModelWeightsShapes)
{
	ModelInstance = NNEModel->CreateModelInstanceCPU();
}

FMLLevelSetNeuralInference& FMLLevelSetNeuralInference::operator=(const FMLLevelSetNeuralInference& Other)
{
	NNEModel = Other.NNEModel;
	ModelWeightsShapes = Other.ModelWeightsShapes;
	ModelInstance = NNEModel->CreateModelInstanceCPU();
	return *this;
}

FMLLevelSetNeuralInference* FMLLevelSetNeuralInference::Copy() const
{
	FMLLevelSetNeuralInference* CopyInstance = new FMLLevelSetNeuralInference();
	CopyInstance->NNEModel = NNEModel;
	CopyInstance->ModelWeightsShapes = ModelWeightsShapes;
	CopyInstance->ModelInstance = NNEModel->CreateModelInstanceCPU();
	return CopyInstance;
}

void FMLLevelSetNeuralInference::RunInference(TArray<float, TAlignedHeapAllocator<64>>& InputData, TArray<float, TAlignedHeapAllocator<64>>& OutputData, 
	uint32 SingleInputSize, uint32 SingleOutputSize, TArray<TArray<float, TAlignedHeapAllocator<64>>>& ModelWeightsIn) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMLLevelSetNeuralInference_RunInference);

	if (ModelInstance.IsValid())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FMLLevelSetNeuralInference_RunInference_Run);
		const uint32 NumberOfInputs = uint32(InputData.Num()) / SingleInputSize;
		TArray<uint32> InputShapeData = { NumberOfInputs , SingleInputSize };
		TArray< UE::NNE::FTensorShape> InputTensorShapes;
		InputTensorShapes.SetNum(ModelWeightsShapes.Num() + 1); //Including Input
		InputTensorShapes[0] = UE::NNE::FTensorShape::Make(InputShapeData);
		for (int32 Idx = 0; Idx < ModelWeightsShapes.Num(); Idx++)
		{
			InputTensorShapes[Idx + 1] = UE::NNE::FTensorShape::Make(ModelWeightsShapes[Idx]);
		}
		ModelInstance->SetInputTensorShapes(InputTensorShapes);

		TArray<uint32> OutputShapeData = { NumberOfInputs, SingleOutputSize };
		TArray< UE::NNE::FTensorShape > OutputTensorShapes = { UE::NNE::FTensorShape::Make(OutputShapeData) };

		TArray<UE::NNE::FTensorBindingCPU> InputBindings;
		TArray<UE::NNE::FTensorBindingCPU> OutputBindings;

		InputBindings.SetNumZeroed(ModelWeightsIn.Num() + 1);
		InputBindings[0].Data = InputData.GetData();
		InputBindings[0].SizeInBytes = InputData.Num() * sizeof(float);
		for (int32 Index = 0; Index < ModelWeightsIn.Num(); Index++)
		{
			InputBindings[Index + 1].Data = ModelWeightsIn[Index].GetData();
			InputBindings[Index + 1].SizeInBytes = ModelWeightsIn[Index].Num() * sizeof(float);
		}
		OutputBindings.SetNumZeroed(1);
		OutputBindings[0].Data = OutputData.GetData();
		OutputBindings[0].SizeInBytes = OutputData.Num() * sizeof(float);

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FMLLevelSetNeuralInference_RunInference_ModelInstanceRunSync);
			ModelInstance->RunSync(InputBindings, OutputBindings);
		}
	}
	else
	{
		UE_LOGF(LogTemp, Error, "FMLLevelSetNeuralInference::RunInference() - ModelInstance us not valid");
	}
}

bool FMLLevelSetNeuralInference::Serialize(FArchive& Ar)
{
	Ar << ModelWeightsShapes;
	return true;
}

}//end Chaos