// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/MLLevelset.h"
#include "Misc/Paths.h"
#include "Async/ParallelFor.h"
#include <algorithm>
#include <vector>
#include "Chaos/PBDSoftsSolverParticles.h" 
#include "Containers/ContainerAllocationPolicies.h"
#include "NNE.h"  

namespace Chaos
{
static int MLLevelSetUpdatePhiFlag = 0;
static FAutoConsoleVariableRef CVarChaosMLLevelSetUpdatePhiFlag(TEXT("p.MLLevelSet.MLLevelSetUpdatePhiFlag"), MLLevelSetUpdatePhiFlag, TEXT("0(No): Default. 1(Yes): Updates DebugPhi Array for MLLevelset Isocounter Visualization. "));

FMLLevelSet::FMLLevelSet(FMLLevelSetImportData&& MLLevelSetImportData)
	: FImplicitObject(EImplicitObject::HasBoundingBox, ImplicitObjectType::MLLevelSet)
	, ActiveBoneNames(MoveTemp(MLLevelSetImportData.ActiveBoneNames))
	, SignedDistanceScaling(MLLevelSetImportData.SignedDistanceScaling)
	, TrainingGridMin(MLLevelSetImportData.TrainingGridMin)
	, TrainingGridUnitAxesXYZ(MoveTemp(MLLevelSetImportData.TrainingGridAxesXYZ))
	, ActiveBonesRotationComponents(MoveTemp(MLLevelSetImportData.ActiveBonesRotationComponents))
	, ActiveBonesReferenceRotations(MoveTemp(MLLevelSetImportData.ActiveBonesReferenceRotations))
	, ActiveBonesReferenceTranslations(MoveTemp(MLLevelSetImportData.ActiveBonesReferenceTranslations))
{
	TotalNumberOfRotationComponents = 0;
	for (int32 RotationComponentIndex = 0; RotationComponentIndex < ActiveBonesRotationComponents.Num(); RotationComponentIndex++)
	{
		TotalNumberOfRotationComponents += ActiveBonesRotationComponents[RotationComponentIndex].Num();
	}
	ActiveBonesRelativeTransforms.SetNum(ActiveBoneNames.Num());
	check(MLLevelSetImportData.NNEModelDataArr.Num() > 0);
	BuildNNEModel(MLLevelSetImportData.NNEModelDataArr[0].ModelArchitectureActivationNodeSizes, MLLevelSetImportData.NNEModelDataArr[0].NNEModelData, MLLevelSetImportData.NNEModelDataArr[0].MLModelWeightsString,
		NNESignedDistanceModel, SignedDistanceNeuralInferences, SignedDistanceModelWeightsShapes, SignedDistanceModelWeights);
	bUseIncorrectZoneModel = MLLevelSetImportData.NNEModelDataArr.Num() > 1;
	if (bUseIncorrectZoneModel)
	{
		BuildNNEModel(MLLevelSetImportData.NNEModelDataArr[1].ModelArchitectureActivationNodeSizes, MLLevelSetImportData.NNEModelDataArr[1].NNEModelData, MLLevelSetImportData.NNEModelDataArr[1].MLModelWeightsString,
			NNEIncorrectZoneModel, IncorrectZoneNeuralInferences, IncorrectZoneModelWeightsShapes, IncorrectZoneModelWeights);
	}

	FVec3 TrainingGridMax = (FVec3)(TrainingGridUnitAxesXYZ[0] + TrainingGridUnitAxesXYZ[1] + TrainingGridUnitAxesXYZ[2] + TrainingGridMin);
	LocalBoundingBox = Chaos::FAABB3(Chaos::FVec3(TrainingGridMin), TrainingGridMax);
	for (int32 CornerIndex = 0; CornerIndex < 3; CornerIndex++) 
	{
		LocalBoundingBox.GrowToInclude((FVec3)TrainingGridUnitAxesXYZ[CornerIndex] + TrainingGridMin);
		LocalBoundingBox.GrowToInclude(TrainingGridMax - (FVec3)TrainingGridUnitAxesXYZ[CornerIndex]);
	}
	ProcessTrainingGridAxesVectors();

	//Create Debug Phi and Debug Grid
	const Chaos::TVec3<int32> DebugChaosGridDim(MLLevelSetImportData.DebugGridResolution[0], MLLevelSetImportData.DebugGridResolution[1], MLLevelSetImportData.DebugGridResolution[2]);
	DebugGrid = TUniformGrid<Chaos::FReal, 3>(LocalBoundingBox.Min(), LocalBoundingBox.Max(), DebugChaosGridDim);
	DebugPhi = TArrayND<Chaos::FReal, 3>(DebugChaosGridDim);
}

CHAOS_API FMLLevelSet::FMLLevelSet(TObjectPtr<UNNEModelData> InNNESignedDistanceModelData, TObjectPtr<UNNEModelData> InNNEIncorrectZoneModel)
	: FImplicitObject(EImplicitObject::HasBoundingBox, ImplicitObjectType::MLLevelSet)
{
	BuildNNEModel(InNNESignedDistanceModelData, NNESignedDistanceModel);
	BuildNNEModel(InNNEIncorrectZoneModel, NNEIncorrectZoneModel);
}

FMLLevelSet::FMLLevelSet(FMLLevelSet&& Other)
	: FImplicitObject(EImplicitObject::HasBoundingBox, ImplicitObjectType::MLLevelSet)
	, ActiveBoneNames(MoveTemp(Other.ActiveBoneNames))
	, ActiveBonesRelativeTransforms(MoveTemp(Other.ActiveBonesRelativeTransforms))

	, NNESignedDistanceModel(Other.NNESignedDistanceModel)
	, SignedDistanceNeuralInferences(MoveTemp(Other.SignedDistanceNeuralInferences))
	, SignedDistanceModelWeights(MoveTemp(Other.SignedDistanceModelWeights))
	, SignedDistanceModelWeightsShapes(MoveTemp(Other.SignedDistanceModelWeightsShapes))

	, NNEIncorrectZoneModel(Other.NNEIncorrectZoneModel)
	, IncorrectZoneNeuralInferences(Other.IncorrectZoneNeuralInferences)
	, IncorrectZoneModelWeights(MoveTemp(Other.IncorrectZoneModelWeights))
	, IncorrectZoneModelWeightsShapes(MoveTemp(Other.IncorrectZoneModelWeightsShapes))
	, bUseIncorrectZoneModel(Other.bUseIncorrectZoneModel)

	, LocalBoundingBox(MoveTemp(Other.LocalBoundingBox))
	, SignedDistanceScaling(Other.SignedDistanceScaling)
	, TrainingGridMin(Other.TrainingGridMin)
	, TrainingGridUnitAxesXYZ(MoveTemp(Other.TrainingGridUnitAxesXYZ))
	, TrainingGridAxesLengthsXYZ(MoveTemp(Other.TrainingGridAxesLengthsXYZ))
	, TotalNumberOfRotationComponents(Other.TotalNumberOfRotationComponents)
	, ActiveBonesRotationComponents(MoveTemp(Other.ActiveBonesRotationComponents))
	, ActiveBonesReferenceRotations(MoveTemp(Other.ActiveBonesReferenceRotations))
	, ActiveBonesReferenceTranslations(MoveTemp(Other.ActiveBonesReferenceTranslations))

	, DebugGrid(MoveTemp(Other.DebugGrid))
	, DebugPhi(MoveTemp(Other.DebugPhi))
{}

FMLLevelSet::~FMLLevelSet()
{}


Chaos::FImplicitObjectPtr FMLLevelSet::CopyGeometry() const
{
	FMLLevelSet* Copy = new FMLLevelSet();

	Copy->ActiveBoneNames = ActiveBoneNames;
	Copy->ActiveBonesRelativeTransforms = ActiveBonesRelativeTransforms;

	Copy->NNESignedDistanceModel = NNESignedDistanceModel;
	Copy->SignedDistanceNeuralInferences = SignedDistanceNeuralInferences;
	Copy->SignedDistanceModelWeights = SignedDistanceModelWeights;
	Copy->SignedDistanceModelWeightsShapes = SignedDistanceModelWeightsShapes;

	Copy->NNEIncorrectZoneModel = NNEIncorrectZoneModel;
	Copy->IncorrectZoneNeuralInferences = IncorrectZoneNeuralInferences;
	Copy->IncorrectZoneModelWeights = IncorrectZoneModelWeights;
	Copy->IncorrectZoneModelWeightsShapes = IncorrectZoneModelWeightsShapes;
	Copy->bUseIncorrectZoneModel = bUseIncorrectZoneModel;

	Copy->LocalBoundingBox = LocalBoundingBox;
	Copy->SignedDistanceScaling = SignedDistanceScaling;
	Copy->TrainingGridMin = TrainingGridMin;
	Copy->TrainingGridUnitAxesXYZ = TrainingGridUnitAxesXYZ;
	Copy->TrainingGridAxesLengthsXYZ = TrainingGridAxesLengthsXYZ;
	Copy->TotalNumberOfRotationComponents = TotalNumberOfRotationComponents;
	Copy->ActiveBonesRotationComponents = ActiveBonesRotationComponents;
	Copy->ActiveBonesReferenceRotations = ActiveBonesReferenceRotations;
	Copy->ActiveBonesReferenceTranslations = ActiveBonesReferenceTranslations;

	Copy->DebugGrid = DebugGrid;
	Copy->DebugPhi.Copy(DebugPhi);

	return Chaos::FImplicitObjectPtr(Copy);
}

Chaos::FImplicitObjectPtr FMLLevelSet::CopyGeometryWithScale(const FVec3& Scale) const
{
	FMLLevelSet* Copy = new FMLLevelSet();

	Copy->ActiveBoneNames = ActiveBoneNames;
	Copy->ActiveBonesRelativeTransforms = ActiveBonesRelativeTransforms;

	Copy->NNESignedDistanceModel = NNESignedDistanceModel;
	Copy->SignedDistanceNeuralInferences = SignedDistanceNeuralInferences;
	Copy->SignedDistanceModelWeights = SignedDistanceModelWeights;
	Copy->SignedDistanceModelWeightsShapes = SignedDistanceModelWeightsShapes;

	Copy->NNEIncorrectZoneModel = NNEIncorrectZoneModel;
	Copy->IncorrectZoneNeuralInferences = IncorrectZoneNeuralInferences;
	Copy->IncorrectZoneModelWeights = IncorrectZoneModelWeights;
	Copy->IncorrectZoneModelWeightsShapes = IncorrectZoneModelWeightsShapes;
	Copy->bUseIncorrectZoneModel = bUseIncorrectZoneModel;

	Copy->LocalBoundingBox = LocalBoundingBox;
	Copy->SignedDistanceScaling = SignedDistanceScaling;
	Copy->TrainingGridMin = TrainingGridMin;
	Copy->TrainingGridUnitAxesXYZ = TrainingGridUnitAxesXYZ;
	Copy->TrainingGridAxesLengthsXYZ = TrainingGridAxesLengthsXYZ;
	Copy->TotalNumberOfRotationComponents = TotalNumberOfRotationComponents;
	Copy->ActiveBonesRotationComponents = ActiveBonesRotationComponents;
	Copy->ActiveBonesReferenceRotations = ActiveBonesReferenceRotations;
	Copy->ActiveBonesReferenceTranslations = ActiveBonesReferenceTranslations;

	Copy->DebugGrid = DebugGrid;
	Copy->DebugPhi.Copy(DebugPhi);

	return MakeImplicitObjectPtr<TImplicitObjectScaled<FMLLevelSet>>(Copy, Scale);
}

/** This function uses DebugPhi. Is there a better way to do this, which uses the network directly? */
void FMLLevelSet::GetZeroIsosurfaceGridCellFaces(TArray<FVector3f>& Vertices, TArray<FIntVector>& Tris) const
{
	const TVector<int32, 3> Cells = DebugGrid.Counts();
	const FVector3d Dx(DebugGrid.Dx());

	for (int i = 0; i < Cells.X - 1; ++i)
	{
		for (int j = 0; j < Cells.Y - 1; ++j)
		{
			for (int k = 0; k < Cells.Z - 1; ++k)
			{
				const double Sign = FMath::Sign(DebugPhi(i, j, k));
				const double SignNextI = FMath::Sign(DebugPhi(i + 1, j, k));
				const double SignNextJ = FMath::Sign(DebugPhi(i, j + 1, k));
				const double SignNextK = FMath::Sign(DebugPhi(i, j, k + 1));

				const FVector3d CellMin = DebugGrid.MinCorner() + Dx * FVector3d(i, j, k);

				if (Sign > SignNextI)
				{
					const int32 V0 = Vertices.Emplace(CellMin + Dx * FVector3d(1, 0, 0));
					const int32 V1 = Vertices.Emplace(CellMin + Dx * FVector3d(1, 1, 0));
					const int32 V2 = Vertices.Emplace(CellMin + Dx * FVector3d(1, 1, 1));
					const int32 V3 = Vertices.Emplace(CellMin + Dx * FVector3d(1, 0, 1));
					Tris.Emplace(FIntVector(V0, V1, V2));
					Tris.Emplace(FIntVector(V2, V3, V0));
				}
				else if (Sign < SignNextI)
				{
					const int32 V0 = Vertices.Emplace(CellMin + Dx * FVector3d(1, 0, 0));
					const int32 V1 = Vertices.Emplace(CellMin + Dx * FVector3d(1, 1, 0));
					const int32 V2 = Vertices.Emplace(CellMin + Dx * FVector3d(1, 1, 1));
					const int32 V3 = Vertices.Emplace(CellMin + Dx * FVector3d(1, 0, 1));
					Tris.Emplace(FIntVector(V0, V2, V1));
					Tris.Emplace(FIntVector(V2, V0, V3));
				}


				if (Sign > SignNextJ)
				{
					const int32 V0 = Vertices.Emplace(CellMin + Dx * FVector3d(0, 1, 0));
					const int32 V1 = Vertices.Emplace(CellMin + Dx * FVector3d(1, 1, 0));
					const int32 V2 = Vertices.Emplace(CellMin + Dx * FVector3d(1, 1, 1));
					const int32 V3 = Vertices.Emplace(CellMin + Dx * FVector3d(0, 1, 1));
					Tris.Emplace(FIntVector(V0, V2, V1));
					Tris.Emplace(FIntVector(V2, V0, V3));
				}
				else if (Sign < SignNextJ)
				{
					const int32 V0 = Vertices.Emplace(CellMin + Dx * FVector3d(0, 1, 0));
					const int32 V1 = Vertices.Emplace(CellMin + Dx * FVector3d(1, 1, 0));
					const int32 V2 = Vertices.Emplace(CellMin + Dx * FVector3d(1, 1, 1));
					const int32 V3 = Vertices.Emplace(CellMin + Dx * FVector3d(0, 1, 1));
					Tris.Emplace(FIntVector(V0, V1, V2));
					Tris.Emplace(FIntVector(V2, V3, V0));
				}

				if (Sign > SignNextK)
				{
					const int32 V0 = Vertices.Emplace(CellMin + Dx * FVector3d(0, 0, 1));
					const int32 V1 = Vertices.Emplace(CellMin + Dx * FVector3d(1, 0, 1));
					const int32 V2 = Vertices.Emplace(CellMin + Dx * FVector3d(1, 1, 1));
					const int32 V3 = Vertices.Emplace(CellMin + Dx * FVector3d(0, 1, 1));
					Tris.Emplace(FIntVector(V0, V1, V2));
					Tris.Emplace(FIntVector(V2, V3, V0));
				}
				else if (Sign < SignNextK)
				{
					const int32 V0 = Vertices.Emplace(CellMin + Dx * FVector3d(0, 0, 1));
					const int32 V1 = Vertices.Emplace(CellMin + Dx * FVector3d(1, 0, 1));
					const int32 V2 = Vertices.Emplace(CellMin + Dx * FVector3d(1, 1, 1));
					const int32 V3 = Vertices.Emplace(CellMin + Dx * FVector3d(0, 1, 1));
					Tris.Emplace(FIntVector(V0, V2, V1));
					Tris.Emplace(FIntVector(V2, V0, V3));
				}
			}
		}
	}
}

/** This function uses DebugPhi. Is there a better way to do this, which uses the network directly? */
void FMLLevelSet::GetInteriorCells(TArray<TVec3<int32>>& InteriorCells, const FReal InteriorThreshold) const
{
	InteriorCells.Reset();

	const TVector<int32, 3> Cells = DebugGrid.Counts();

	for (int i = 0; i < Cells.X; ++i)
	{
		for (int j = 0; j < Cells.Y; ++j)
		{
			for (int k = 0; k < Cells.Z; ++k)
			{
				const FReal Value = DebugPhi(i, j, k);
				if (Value < InteriorThreshold)
				{
					InteriorCells.Emplace(i, j, k);
				}
			}
		}
	}
}

/** This function should not be called. MLLevelSets are only BatchPhiWithNormal friendly */
FReal FMLLevelSet::SignedDistance(const FVec3& x) const
{
	UE_LOGF(LogChaos, Error, "FMLLevelSet::PhiWithNormal cannot be used for single queries. Use FMLLevelSet::BatchPhiWithNormal() instead.");
	return UE_DOUBLE_BIG_NUMBER;
}

void FMLLevelSet::CreatePhiFromMLModel()
{	
	const TVector<int32, 3> Cells = DebugGrid.Counts();
	TArray<float, TAlignedHeapAllocator<64>> OutputDataSignedDistances;
	TArray<float, TAlignedHeapAllocator<64>> InputDataLocationsMS;
	const uint32 ModelInputShapeSize = 3;
	const uint32 ModelOutputShapeSize = 1;
	const uint32 ModelOutputSignedDistanceIndex = 0;

	TArray<TArray<float, TAlignedHeapAllocator<64>>> MLWeightsIn;
	ComputeSignedDistanceNetworkWeightsInput(MLWeightsIn);

	InputDataLocationsMS.SetNum(ModelInputShapeSize * DebugGrid.GetNumCells());
	OutputDataSignedDistances.SetNum(ModelOutputShapeSize * DebugGrid.GetNumCells());

	TArray<FVector3f> TrainingGridVectorsScaled;
	TrainingGridVectorsScaled.SetNum(3);
	for (int32 Index = 0; Index < 3; Index++)
	{
		TrainingGridVectorsScaled[Index] = TrainingGridUnitAxesXYZ[Index] / TrainingGridAxesLengthsXYZ[Index];
	}

	for (int32 i = 0; i < DebugGrid.Counts()[0]; i++)
	{
		for (int32 j = 0; j < DebugGrid.Counts()[1]; j++)
		{
			for (int32 k = 0; k < DebugGrid.Counts()[2]; k++)
			{
				TVector<int32, 3> Index = { i, j, k };
				const int32 ClosestActiveBoneIndex = GetClosestActiveBoneIndex(DebugGrid.Location(Index));
				const FVector3f LocalGridCornerShift = (FVector3f)ActiveBonesRelativeTransforms[ClosestActiveBoneIndex].GetTranslation() - (FVector3f)ActiveBonesReferenceTranslations[ClosestActiveBoneIndex];
				const FVector3f LocalGridCornerShifted = TrainingGridMin + LocalGridCornerShift;

				TVector<FRealSingle, 3> LocationMSShifted = DebugGrid.Location(Index) - LocalGridCornerShifted;
				for (int32 ModelInputLocationIndex = 0; ModelInputLocationIndex < ModelInputShapeSize; ModelInputLocationIndex++)
				{
					InputDataLocationsMS[ModelInputShapeSize * DebugGrid.FlatIndex(Index) + ModelInputLocationIndex] = TrainingGridVectorsScaled[ModelInputLocationIndex][0] * LocationMSShifted[0] + TrainingGridVectorsScaled[ModelInputLocationIndex][1] * LocationMSShifted[1] + TrainingGridVectorsScaled[ModelInputLocationIndex][2] * LocationMSShifted[2];
				}
			}
		}
	}

	if (SignedDistanceNeuralInferences[0].IsValid())
	{
		SignedDistanceNeuralInferences[0].RunInference(InputDataLocationsMS, OutputDataSignedDistances, ModelInputShapeSize, ModelOutputShapeSize, MLWeightsIn);
	}

	for (int32 i = 0; i < DebugGrid.Counts()[0]; i++)
	{
		for (int32 j = 0; j < DebugGrid.Counts()[1]; j++)
		{
			for (int32 k = 0; k < DebugGrid.Counts()[2]; k++)
			{
				TVector<int32, 3> Index = { i, j, k };
				int32 FlatIndex = DebugGrid.FlatIndex(Index);
				DebugPhi(i, j, k) = SignedDistanceScaling * OutputDataSignedDistances[ModelOutputShapeSize * FlatIndex + ModelOutputSignedDistanceIndex];
			}
		}
	}
}


void FMLLevelSet::UpdateActiveBonesRelativeTransforms(TArray<FTransform>& InActiveBonesRelativeTransforms)
{
	bool bIsFTransformArraysDifferent = IsFTransformArraysDifferent(ActiveBonesRelativeTransforms, InActiveBonesRelativeTransforms);
	
	if (bIsFTransformArraysDifferent)
	{
		ActiveBonesRelativeTransforms = MoveTemp(InActiveBonesRelativeTransforms);
		
		// Phi is created for each timestep for debug drawing
		if (MLLevelSetUpdatePhiFlag == 1)
		{
			CreatePhiFromMLModel();
		}
	}
}

void FMLLevelSet::UpdateActiveBonesRelativeTransformsAndUpdateDebugPhi(TArray<FTransform>& InActiveBonesRelativeTransforms)
{
	bool bIsFTransformDifferent = IsFTransformArraysDifferent(ActiveBonesRelativeTransforms, InActiveBonesRelativeTransforms);
	ActiveBonesRelativeTransforms = MoveTemp(InActiveBonesRelativeTransforms);

	// Update DebugPhi only if FTransform is updated
	if (bIsFTransformDifferent)
	{
		CreatePhiFromMLModel();
	}
}

// Adds more MLLevelSetNeuralInference to allow multi-treating.
void FMLLevelSet::UpdateNeuralInferencesNumber(const int32 InNeuralInferencesNumber)
{
	if (SignedDistanceNeuralInferences.Num() < InNeuralInferencesNumber)
	{
		for (int32 Index = SignedDistanceNeuralInferences.Num(); Index < InNeuralInferencesNumber; Index++)
		{
			SignedDistanceNeuralInferences.Add(FMLLevelSetNeuralInference(NNESignedDistanceModel, SignedDistanceModelWeightsShapes));
			if (bUseIncorrectZoneModel)
			{
				IncorrectZoneNeuralInferences.Add(FMLLevelSetNeuralInference(NNEIncorrectZoneModel, IncorrectZoneModelWeightsShapes));
			}
		}
	}
}

// Body Part is trained over a grid/BoundingBox created outside of the engine
// This training grid might not align with the coordinate system axes of the engine
// TrainingGridVectors are used for mapping from training coordinate system to engine coordinate system
// Important Note: If training uses MAYA for generating the dataset, do not forget to negate the Y-axis coordinate before importing the DataTable
// I.e., UE.Y = - Maya.Y
void FMLLevelSet::ProcessTrainingGridAxesVectors()
{
	//check if training grid vectors are initialized
	check(TrainingGridUnitAxesXYZ.Num() == 3);

	TrainingGridAxesLengthsXYZ.Empty();
	TrainingGridAxesLengthsXYZ.SetNum(3);

	for (int32 Index = 0; Index < 3; Index++)
	{
		TrainingGridAxesLengthsXYZ[Index] = float(TrainingGridUnitAxesXYZ[Index].Length());
		TrainingGridUnitAxesXYZ[Index] = TrainingGridUnitAxesXYZ[Index] / TrainingGridAxesLengthsXYZ[Index];
	}
}

// Loads MLModelWeights From an FString
// TArray<TArray<float>> MLModelWeights = {W1,..,Wn} is tokanized into FString as 
// "W1_0,W1_1,...,W1_k1|W2_0,W2_1,...,W2_k2|...|Wn_0,Wn_1,...,Wn_kn" 
void FMLLevelSet::LoadMLModelWeightsFromString(const FString& MLModelWeightsString, TArray<TArray<float>>& ModelWeightArray)
{
	TArray<FString> MLModelWeightsArrayStringsSeperatedByBar;
	MLModelWeightsString.ParseIntoArray(MLModelWeightsArrayStringsSeperatedByBar, TEXT("|"), true);

	ModelWeightArray.Empty();
	ModelWeightArray.SetNum(MLModelWeightsArrayStringsSeperatedByBar.Num());
	for (int32 i = 0; i < MLModelWeightsArrayStringsSeperatedByBar.Num(); i++)
	{
		TArray<FString> MLModelWeightsStringsSeperatedByComma;
		MLModelWeightsArrayStringsSeperatedByBar[i].ParseIntoArray(MLModelWeightsStringsSeperatedByComma,TEXT(","),true);
		ModelWeightArray[i].Empty();
		ModelWeightArray[i].SetNum(MLModelWeightsStringsSeperatedByComma.Num());
		
		for (int32 j = 0; j < ModelWeightArray[i].Num(); j++)
		{
			ModelWeightArray[i][j]= FCString::Atof(*MLModelWeightsStringsSeperatedByComma[j]);
		}
	}
}

void FMLLevelSet::BuildNNEModel(const TArray<int32>& ModelArchitectureActivationNodeSizes, TObjectPtr<UNNEModelData> NNEModelData, const FString& ModelWeightsString,
	TSharedPtr<UE::NNE::IModelCPU>& NNEModel, TArray<FMLLevelSetNeuralInference>& NeuralInferences, TArray<TArray<int32>>& ModelWeightsShapes, TArray<TArray<float>>& ModelWeightArray)
{
	// This is specific to weight-updated MLP types of networks.
	// (BigToDo) Update this part to accept different types of networks.
	ModelWeightsShapes.Empty();
	for (int32 i = 0; i < ModelArchitectureActivationNodeSizes.Num() - 1; i++)
	{
		ModelWeightsShapes.Add(TArray<int32>({ ModelArchitectureActivationNodeSizes[i],ModelArchitectureActivationNodeSizes[i + 1] }));
		ModelWeightsShapes.Add(TArray<int32>({ ModelArchitectureActivationNodeSizes[i + 1] }));
	}

	//Create Model Instance and NNE Neural Inferences
	TWeakInterfacePtr<INNERuntimeCPU> Runtime = UE::NNE::GetRuntime<INNERuntimeCPU>(FString("NNERuntimeIREECpu"));
	if (Runtime.IsValid())
	{
		NNEModel = Runtime->CreateModelCPU(NNEModelData);
		if (NNEModel.IsValid())
		{
			NeuralInferences.Empty();
			NeuralInferences.Add(FMLLevelSetNeuralInference(NNEModel, ModelWeightsShapes));
		}
		else
		{
			UE_LOGF(LogChaos, Error, "MLLevelSet::BuildNNEModel - Model is not valid.");
		}
	}
	else
	{
		UE_LOGF(LogChaos, Error, "MLLevelSet::BuildNNEModel - NNE Runtime is NOT valid.");
	}

	LoadMLModelWeightsFromString(ModelWeightsString, ModelWeightArray);
}

// Only Build NNE Model. Used for serialization
void FMLLevelSet::BuildNNEModel(TObjectPtr<UNNEModelData> InNNEModelData, TSharedPtr<UE::NNE::IModelCPU>& NNEModel)
{
	TWeakInterfacePtr<INNERuntimeCPU> Runtime = UE::NNE::GetRuntime<INNERuntimeCPU>(FString("NNERuntimeIREECpu"));
	if (Runtime.IsValid())
	{
		NNEModel = Runtime->CreateModelCPU(InNNEModelData);
	}
	else
	{
		UE_LOGF(LogChaos, Error, "MLLevelSet::BuildNNEModel - NNE Runtime is NOT valid.");
	}
}

const bool FMLLevelSet::IsFTransformArraysDifferent(TArray<FTransform>& FTransformArr1, TArray<FTransform>& FTransformArr2, FReal Tol) const
{
	check(FTransformArr1.Num() == FTransformArr2.Num());
	for (int32 Index = 0; Index < FTransformArr1.Num(); Index++)
	{
		FVector TranslationDelta = FTransformArr1[Index].GetTranslation() - FTransformArr2[Index].GetTranslation() ;
		if (TranslationDelta.GetAbsMax() > Tol)
		{
			return true;
		}
		FRotator RotationDelta = FRotator(FTransformArr1[Index].GetRotation()) - FRotator(FTransformArr2[Index].GetRotation());
		if (!RotationDelta.IsNearlyZero(Tol))
		{
			return true;
		}
	}
	return false;
}

FReal FMLLevelSet::PhiWithNormal(const FVec3& x, FVec3& Normal) const
{
	UE_LOGF(LogChaos, Error, "FMLLevelSet::PhiWithNormal() cannot be used for single queries. Use FMLLevelSet::BatchPhiWithNormal() instead.");
	return UE_DOUBLE_BIG_NUMBER;
}

void FMLLevelSet::ComputeSignedDistanceNetworkWeightsInput(TArray<TArray<float, TAlignedHeapAllocator<64>>>& NetworkWeightsInput) const
{
	ComputeWeightsInput(ActiveBonesRelativeTransforms, SignedDistanceModelWeights, SignedDistanceModelWeightsShapes, NetworkWeightsInput);
}

void FMLLevelSet::ComputeIncorrectZoneNetworkWeightsInput(TArray<TArray<float, TAlignedHeapAllocator<64>>>& NetworkWeightsInput) const
{
	ComputeWeightsInput(ActiveBonesRelativeTransforms, IncorrectZoneModelWeights, IncorrectZoneModelWeightsShapes, NetworkWeightsInput);
}


void FMLLevelSet::ComputeWeightsInput(const TArray<FTransform>& RelativeBoneTransformationsInput, const TArray<TArray<float>>& NetworkWeights, const TArray<TArray<int32>>& NetworkWeightsShapes, TArray<TArray<float, TAlignedHeapAllocator<64>>>& NetworkWeightsInput) const
{
	checkSlow(RelativeBoneTransformationsInput.Num() == GetNumberOfActiveBones());
	NetworkWeightsInput.Empty();
	NetworkWeightsInput.SetNum(NetworkWeights.Num());
	float HalfRotatationAngle = 180.0;

	TArray<TArray<float>> JointRotationInputAnglesMod360;
	JointRotationInputAnglesMod360.SetNum(RelativeBoneTransformationsInput.Num());
	for (int32 RotationIndex = 0; RotationIndex < JointRotationInputAnglesMod360.Num(); RotationIndex++)
	{
		FRotator Rot(RelativeBoneTransformationsInput[RotationIndex].GetRotation());
		JointRotationInputAnglesMod360[RotationIndex] = { (float)(Rot.Roll - ActiveBonesReferenceRotations[RotationIndex].X), (float)(Rot.Pitch - ActiveBonesReferenceRotations[RotationIndex].Y), (float)(Rot.Yaw - ActiveBonesReferenceRotations[RotationIndex].Z)};
		// Put Joint Rotation Angles in (-180,180] in mod 360
		for (int i = 0; i < 3; i++)
		{
			while (JointRotationInputAnglesMod360[RotationIndex][i] <= -HalfRotatationAngle)
			{
				JointRotationInputAnglesMod360[RotationIndex][i] += 2*HalfRotatationAngle;
			}
			while (JointRotationInputAnglesMod360[RotationIndex][i] > HalfRotatationAngle)
			{
				JointRotationInputAnglesMod360[RotationIndex][i] -= 2 * HalfRotatationAngle;
			}
		}
	}	

	TArray<float> MLRotationAngleInput;
	MLRotationAngleInput.Reserve(TotalNumberOfRotationComponents);

	for (int32 RotIndex = 0; RotIndex < ActiveBonesRotationComponents.Num(); RotIndex++)
	{
		for (int32 RotationComponent : ActiveBonesRotationComponents[RotIndex])
		{
			MLRotationAngleInput.Add(float(JointRotationInputAnglesMod360[RotIndex][RotationComponent] / 360.0));
		}
	}
	
	const int32 NumberOfAngleParameters = MLRotationAngleInput.Num();
	const int32 NumberOfWeightVariables = NumberOfAngleParameters + 1;
	for (int32 WI = 0; WI < NetworkWeights.Num(); WI++)
	{
		NetworkWeightsInput[WI].SetNum(NetworkWeights[WI].Num() / NumberOfWeightVariables);
		for (int32 WJ = 0; WJ < NetworkWeightsInput[WI].Num(); WJ++)
		{
			int32 WidthSize = NetworkWeightsShapes[WI][NetworkWeightsShapes[WI].Num() - 1];
			int32 WJ1 = WJ / WidthSize;
			int32 WJ2 = WJ % WidthSize;
			NetworkWeightsInput[WI][WJ] = NetworkWeights[WI][WJ1 * WidthSize * NumberOfWeightVariables + WJ2 + (NumberOfWeightVariables - 1) * WidthSize];
			for (int32 InputWeightIndex = 0; InputWeightIndex < NumberOfAngleParameters; InputWeightIndex++)
			{
				NetworkWeightsInput[WI][WJ] += MLRotationAngleInput[InputWeightIndex] * NetworkWeights[WI][WJ1 * WidthSize * NumberOfWeightVariables + WJ2 + InputWeightIndex * WidthSize];
			}
		}
	}
}

void FMLLevelSet::BatchPhiWithNormal(const TConstArrayView<Softs::FPAndInvM> PAndInvMArray,
	const Softs::FSolverRigidTransform3& SolverToThis, TArray<Softs::FSolverReal>& OutBatchPhis, TArray<Softs::FSolverVec3>& OutBatchNormals,
	const Chaos::Softs::FSolverReal CollisionThickness, const int32 MLLevelsetThread, const int32 BatchBegin, const int32 BatchEnd) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMLLevelSet_BatchPhiWithNormal);
	const uint32 ModelInputShapeSize = 3;  // XYZ coordinates wrt local grid
	const uint32 ModelOutputShapeSize = 1; // SignedDistance Output
	const Chaos::Softs::FSolverReal MaxPhiValue = 2 * SignedDistanceScaling;
	const float DeltaForFiniteDifferenceForNormalCalc = SignedDistanceScaling / float(100); //delta for finite differencing

	const int32 NumParticles = BatchEnd - BatchBegin;
	const Softs::FPAndInvM* const PAndInvM = PAndInvMArray.GetData();
	Softs::FSolverReal* const Phis = OutBatchPhis.GetData();
	Softs::FSolverVec3* const NormalsOut = OutBatchNormals.GetData();

	//Query only the phi values on the cloth
	TArray<float, TAlignedHeapAllocator<64>> MLInputForPhis;
	MLInputForPhis.SetNum(NumParticles * ModelInputShapeSize);
	TArray<float, TAlignedHeapAllocator<64>> MLOutputForPhis;

	TArray<int32> ParticlesInsideTheGridIndexes;
	ParticlesInsideTheGridIndexes.Reserve(NumParticles);
	TArray<TArray<float, TAlignedHeapAllocator<64>>> MLWeightsIn;
	ComputeSignedDistanceNetworkWeightsInput(MLWeightsIn);

	TArray<FVector3f> TrainingGridVectorsScaled;
	TrainingGridVectorsScaled.SetNum(3);
	for (int32 Index = 0; Index < 3; Index++)
	{
		TrainingGridVectorsScaled[Index] = TrainingGridUnitAxesXYZ[Index] / TrainingGridAxesLengthsXYZ[Index];
	}

	int32 InteriorParticlesIndex = 0;
	for (int32 Index = BatchBegin; Index < BatchEnd; Index++)
	{
		OutBatchPhis[Index] = MaxPhiValue;

		if (PAndInvM[Index].InvM == (Softs::FSolverReal)0.)
		{
			continue;
		}

		const FVec3 RigidSpacePosition(SolverToThis.TransformPositionNoScale(PAndInvM[Index].P));

		// only query points inside of the grid 
		if (LocalBoundingBox.Contains(RigidSpacePosition))
		{
			const int32 ClosestActiveBoneIndex = GetClosestActiveBoneIndex(RigidSpacePosition);
			const FVector3f LocalGridCornerShift = (FVector3f)ActiveBonesRelativeTransforms[ClosestActiveBoneIndex].GetTranslation() - (FVector3f)ActiveBonesReferenceTranslations[ClosestActiveBoneIndex];
			const FVector3f LocalGridCornerShifted = TrainingGridMin + LocalGridCornerShift;

			TVector<FRealSingle, 3> LocationMSShifted = RigidSpacePosition - LocalGridCornerShifted;
			for (int32 j = 0; j < 3; j++)
			{
				MLInputForPhis[InteriorParticlesIndex * ModelInputShapeSize + j] = TrainingGridVectorsScaled[j][0] * LocationMSShifted[0] + TrainingGridVectorsScaled[j][1] * LocationMSShifted[1] + TrainingGridVectorsScaled[j][2] * LocationMSShifted[2];
			}
			ParticlesInsideTheGridIndexes[InteriorParticlesIndex] = Index;
			InteriorParticlesIndex++;
		}
	}

	MLInputForPhis.SetNum(InteriorParticlesIndex * ModelInputShapeSize);
	MLOutputForPhis.SetNum(InteriorParticlesIndex * ModelOutputShapeSize);
	ParticlesInsideTheGridIndexes.SetNum(InteriorParticlesIndex);

	if (SignedDistanceNeuralInferences[MLLevelsetThread].IsValid())
	{
		SignedDistanceNeuralInferences[MLLevelsetThread].RunInference(MLInputForPhis, MLOutputForPhis, ModelInputShapeSize, ModelOutputShapeSize, MLWeightsIn);
	}

	TArray<int32> NegativePhiValueIndices;
	NegativePhiValueIndices.Init(-1, NumParticles);

	// Only Calculate Normals of those particles which are inside of the Body
	int32 NumberOfNegativePhis = 0;
	for (int32 IndexML = 0; IndexML < ParticlesInsideTheGridIndexes.Num(); IndexML++)
	{
		int32 Index = ParticlesInsideTheGridIndexes[IndexML];
		Phis[Index] = SignedDistanceScaling * MLOutputForPhis[IndexML * ModelOutputShapeSize];
		if (Phis[Index] < CollisionThickness)
		{
			NegativePhiValueIndices[NumberOfNegativePhis] = Index;
			NumberOfNegativePhis++;
		}
	}
	NegativePhiValueIndices.SetNum(NumberOfNegativePhis);

	TArray<int32> CorrectZoneIndices;
	int32 NumberOfCorrectZoneIndices = 0;

	if (bUseIncorrectZoneModel)
	{
		TArray<float, TAlignedHeapAllocator<64>> MLInputForIncorrectZone;
		TArray<float, TAlignedHeapAllocator<64>> MLOutputForIncorrectZone;
		const int32 ModelOutputShapeSizeForIncorrectZone = 1;

		MLInputForIncorrectZone.SetNumUninitialized(NegativePhiValueIndices.Num() * ModelInputShapeSize);
		MLOutputForIncorrectZone.SetNumUninitialized(NegativePhiValueIndices.Num() * ModelOutputShapeSizeForIncorrectZone);
		CorrectZoneIndices.SetNumUninitialized(NegativePhiValueIndices.Num());

		for (int32 NegativePhiIndex = 0; NegativePhiIndex < NegativePhiValueIndices.Num(); NegativePhiIndex++)
		{
			int32 Index = NegativePhiValueIndices[NegativePhiIndex];
			const FVec3 PositionLocal(SolverToThis.TransformPositionNoScale(PAndInvM[Index].P));
			const int32 ClosestBoneJointIndex = GetClosestActiveBoneIndex(PositionLocal);
			const FVector3f LocalGridCornerShift = (FVector3f)ActiveBonesRelativeTransforms[ClosestBoneJointIndex].GetTranslation() - (FVector3f)ActiveBonesReferenceTranslations[ClosestBoneJointIndex];
			const FVector3f LocalGridCornerShifted = TrainingGridMin + LocalGridCornerShift;

			TVector<FRealSingle, 3> LocationMSShifted = PositionLocal - LocalGridCornerShifted;
			for (int32 j = 0; j < 3; j++)
			{
				MLInputForIncorrectZone[NegativePhiIndex * ModelInputShapeSize + j] = TrainingGridVectorsScaled[j][0] * LocationMSShifted[0] + TrainingGridVectorsScaled[j][1] * LocationMSShifted[1] + TrainingGridVectorsScaled[j][2] * LocationMSShifted[2];
			}
		}
		TArray<TArray<float, TAlignedHeapAllocator<64>>> NetworkWeightsInputForIncorrectZone;
		ComputeIncorrectZoneNetworkWeightsInput(NetworkWeightsInputForIncorrectZone);
		IncorrectZoneNeuralInferences[MLLevelsetThread].RunInference(MLInputForIncorrectZone, MLOutputForIncorrectZone, ModelInputShapeSize, ModelOutputShapeSizeForIncorrectZone, NetworkWeightsInputForIncorrectZone);

		for (int32 NegativePhiIndex = 0; NegativePhiIndex < NegativePhiValueIndices.Num(); NegativePhiIndex++)
		{
			int32 Index = NegativePhiValueIndices[NegativePhiIndex];
			if (MLOutputForIncorrectZone[NegativePhiIndex] > 0)
			{
				CorrectZoneIndices[NumberOfCorrectZoneIndices] = Index;
				NumberOfCorrectZoneIndices++;
			}
			else
			{
				Phis[Index] = MaxPhiValue;
			}
		}
		CorrectZoneIndices.SetNum(NumberOfCorrectZoneIndices);
	}
	else
	{
		CorrectZoneIndices = MoveTemp(NegativePhiValueIndices);
	}

	TArray<float, TAlignedHeapAllocator<64>> MLInputForNormals;
	MLInputForNormals.SetNum(CorrectZoneIndices.Num() * ModelInputShapeSize * ModelInputShapeSize);
	TArray<float, TAlignedHeapAllocator<64>> MLOutputForNormals;
	MLOutputForNormals.SetNum(CorrectZoneIndices.Num() * ModelInputShapeSize * ModelOutputShapeSize);

	for (int32 CorrectZoneIndex = 0; CorrectZoneIndex < CorrectZoneIndices.Num(); CorrectZoneIndex++)
	{
		int32 Index = CorrectZoneIndices[CorrectZoneIndex];

		const FVec3 RigidSpacePosition(SolverToThis.TransformPositionNoScale(PAndInvM[Index].P));
		const int32 ClosestBoneJointIndex = GetClosestActiveBoneIndex(RigidSpacePosition);
		const FVector3f LocalGridCornerShift = (FVector3f)ActiveBonesRelativeTransforms[ClosestBoneJointIndex].GetTranslation() - (FVector3f)ActiveBonesReferenceTranslations[ClosestBoneJointIndex];
		const FVector3f LocalGridCornerShifted = TrainingGridMin + LocalGridCornerShift;
		TVector<FRealSingle, 3> LocationMSShifted = RigidSpacePosition - LocalGridCornerShifted;

		for (uint32 NormalDirectionIndex = 0; NormalDirectionIndex < 3; NormalDirectionIndex++)
		{
			for (uint32 i = 0; i < ModelInputShapeSize; i++)
			{
				MLInputForNormals[CorrectZoneIndex * ModelInputShapeSize * ModelInputShapeSize + NormalDirectionIndex * ModelInputShapeSize + i] =
					TrainingGridUnitAxesXYZ[i].Dot(LocationMSShifted) / TrainingGridAxesLengthsXYZ[i];
				MLInputForNormals[CorrectZoneIndex * ModelInputShapeSize * ModelInputShapeSize + NormalDirectionIndex * ModelInputShapeSize + i] +=
					DeltaForFiniteDifferenceForNormalCalc * float(TrainingGridUnitAxesXYZ[i][NormalDirectionIndex]) / TrainingGridAxesLengthsXYZ[i];
			}
		}
	}

	if (SignedDistanceNeuralInferences[MLLevelsetThread].IsValid())
	{
		SignedDistanceNeuralInferences[MLLevelsetThread].RunInference(MLInputForNormals, MLOutputForNormals, ModelInputShapeSize, ModelOutputShapeSize, MLWeightsIn);
	}

	for (int32 CorrectZoneIndex = 0; CorrectZoneIndex < CorrectZoneIndices.Num(); CorrectZoneIndex++)
	{
		int32 Index = CorrectZoneIndices[CorrectZoneIndex];
		FVec3 Normal;
		for (int32 i = 0; i < 3; i++)
		{
			Normal[i] = (SignedDistanceScaling * MLOutputForNormals[CorrectZoneIndex * ModelInputShapeSize * ModelOutputShapeSize + ModelOutputShapeSize * i] - Phis[Index]) / DeltaForFiniteDifferenceForNormalCalc;
		}

		constexpr FReal EpsilonForSafeNormalization = 1.0e-8;
		Normal.SafeNormalize(EpsilonForSafeNormalization);
		NormalsOut[Index] = (Softs::FSolverVec3)Normal;
	}	
}
}
