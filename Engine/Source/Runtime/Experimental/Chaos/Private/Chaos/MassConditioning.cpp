// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/MassConditioning.h"

#include "Chaos/ConstraintHandle.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/PBDJointConstraints.h"
#include "Chaos/PBDJointConstraintUtilities.h"

namespace Chaos
{
	FVec3f CalculateInertiaConditioning(const FRealSingle InvM, const FVec3f& InvI, const FVec3f& ConstraintExtents, const FRealSingle MaxDistance, const FRealSingle MaxRotationRatio, const FRealSingle MaxInvInertiaComponentRatio, const FInertiaConditioningTolerances& Tolerances)
	{
		FVec3f InvInertiaScale = FVec3f(1);

		// Sanity check for zero mass or zero dimensions along at least one axis
		// We should never get this unless there are asset errors
		bool bIsValid = 
			(InvM > Tolerances.InvMassTolerance)
			&& (InvI.GetMin() > Tolerances.InvInertiaTolerance)
			&& (ConstraintExtents.GetMin() > Tolerances.ExtentTolerance);

		if (bIsValid)
		{
			// The I,J elements of this matrix can be interpreted the ratio of the rotation correction versus the translation 
			// correction for a constraint arm along the Ith axis and a correction along the Jth axis. It has zero diagonal 
			// because you get no rotation when correcting an error in the same direction as the constraint arm.
			//
			// The math: assume we have a constraint with a constraint arm along the X axis at MaxArm.X. Then assume we need to
			// correct an error, and consider that error to be along each of the principle axes. The corrections perpendicular 
			// to the arm (Y or Z axis) have worst-case rotation versus linear correction. The ratio of rotation correction 
			// to linear correction would be (InvI . R x N x R) / InvM,  where R is the X axis and N is the X, Y or Z axis.
			// For a cube of uniform density with no attached joints, each of these non-zero elements will be the same: 6
			const FVec3f ConstraintExtentsSq = ConstraintExtents * ConstraintExtents;
			const FVec3f MassInertiaRatio = InvI / InvM;
			const FVec3f IRxNxRX = ConstraintExtentsSq.X * FVec3f(0, MassInertiaRatio.Y, MassInertiaRatio.Z);
			const FVec3f IRxNxRY = ConstraintExtentsSq.Y * FVec3f(MassInertiaRatio.X, 0, MassInertiaRatio.Z);
			const FVec3f IRxNxRZ = ConstraintExtentsSq.Z * FVec3f(MassInertiaRatio.X, MassInertiaRatio.Y, 0);

			// Now we need to calculate a scale to apply to each component of the inverse inertia so that each of the elements in
			// the above matrix is greater than some value. E.g., we modify the inertia so that the "worst case" corrections
			// will always have a rotation correction that supplies less that MaxRotationRatio of the total correction.
			// We also reduce the maximum rotation ratio (i.e., increase inertia) for small objects with size less than MaxDistance
			FRealSingle ScaledMaxRotationRatio = MaxRotationRatio;
			if (MaxDistance > 0)
			{
				const FRealSingle MaxRotationRatioScale = ConstraintExtents.GetMax() / MaxDistance;
				const FRealSingle ClampedMaxRotationRatioScale = FMath::Min(MaxRotationRatioScale, FRealSingle(1));
				ScaledMaxRotationRatio = ClampedMaxRotationRatioScale * MaxRotationRatio;
			}

			const FVec3f RotationRatio = FVec3f::Max3(IRxNxRX, IRxNxRY, IRxNxRZ);
			InvInertiaScale = (FVec3f(ScaledMaxRotationRatio) / RotationRatio);

			// We don't ever shrink the inertia. I.e., inv inertia scale must be less than 1
			InvInertiaScale = InvInertiaScale.ComponentwiseMin(FVec3(1));

			// Ensure that all components are at most MaxInvInertiaComponentRatio times the smallest component
			if (MaxInvInertiaComponentRatio > 1)
			{
				const FVec3f ScaledInvI = InvInertiaScale * InvI;
				const FRealSingle MinInvInertia = ScaledInvI.GetMin();
				const FRealSingle MaxInvInertia = MaxInvInertiaComponentRatio * MinInvInertia;
				if (ScaledInvI.X > MaxInvInertia)
				{
					InvInertiaScale.X = MaxInvInertia / ScaledInvI.X;
				}
				if (ScaledInvI.Y > MaxInvInertia)
				{
					InvInertiaScale.Y = MaxInvInertia / ScaledInvI.Y;
				}
				if (ScaledInvI.Z > MaxInvInertia)
				{
					InvInertiaScale.Z = MaxInvInertia / ScaledInvI.Z;
				}

				InvInertiaScale = InvInertiaScale.ComponentwiseMin(FVec3(1));
			}
		}
		
		return InvInertiaScale;
	}

	// Get the extents of all constraints in centre-of-mass space
	FVec3f CalculateCoMConstraintExtents(const FPBDRigidParticleHandle* Rigid)
	{
		const FRigidTransform3 CoMTransform = FRigidTransform3(Rigid->CenterOfMass(), Rigid->RotationOfMass());

		// Calculate the largest constraint arm from collisions and joints.
		// For collisions, we assume contacts at the max extents of the bounds
		// For joints, we use the actual constraint arm of all attached joints
		const FAABB3 CoMBounds = Rigid->LocalBounds().InverseTransformedAABB(CoMTransform);
		const FVec3f ColisionExtents = FReal(0.5) * CoMBounds.Extents();

		FVec3 ConstraintExtents = FVec3(0);
		for (FConstraintHandle* Constraint : Rigid->ParticleConstraints())
		{
			FVec3 ConstraintArm = FVec3(0);
			if (const FPBDJointConstraintHandle* Joint = Constraint->As<FPBDJointConstraintHandle>())
			{
				if (!Chaos::FPBDJointUtilities::GetLinearIsFree(Joint->GetSettings()))
				{
					const TVector<FGeometryParticleHandle*, 2> JointParticles = Joint->GetConstrainedParticles();
					check((Rigid == JointParticles[0]) || (Rigid == JointParticles[1]));
					const int32 JointParticleIndex = (Rigid == JointParticles[0]) ? 0 : 1;
					const FVec3& ActorJointArm = Joint->GetSettings().ConnectorTransforms[JointParticleIndex].GetTranslation();
					ConstraintArm = CoMTransform.InverseTransformPosition(ActorJointArm);
				}
			}

			ConstraintExtents = FVec3::Max(ConstraintExtents, ConstraintArm.GetAbs());
		}

		return FVec3::Max(ColisionExtents, ConstraintExtents);
	}

	FVec3f CalculateParticleInertiaConditioning(const FPBDRigidParticleHandle* Rigid, const FRealSingle MaxDistance, const FRealSingle MaxRotationRatio, const FRealSingle MaxInvInertiaComponentRatio, const FInertiaConditioningTolerances& Tolerances)
	{
		if (Rigid->IsDynamic())
		{
			const FVec3 ConstraintExtents = CalculateCoMConstraintExtents(Rigid);
			return CalculateInertiaConditioning(FRealSingle(Rigid->InvM()), Rigid->InvI(), ConstraintExtents, MaxDistance, MaxRotationRatio, MaxInvInertiaComponentRatio, Tolerances);
		}
		return FVec3f(1);
	}
}