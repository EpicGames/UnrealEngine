// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/MassProperties.h"
//#include "Chaos/Core.h"
#include "Chaos/Rotation.h"
#include "Chaos/Matrix.h"
#include "Chaos/Particles.h"
#include "Chaos/TriangleMesh.h"
#include "Chaos/Utilities.h"
#include "ChaosCheck.h"
#include "Chaos/CastingUtilities.h"

//PRAGMA_DISABLE_OPTIMIZATION

namespace Chaos
{
	template<typename T>
	TRotation<T,3> TransformToLocalSpace(PMatrix<T,3,3>& Inertia)
	{
		TRotation<T,3> FinalRotation;

		// Extract Eigenvalues
		T OffDiagSizeSq = FMath::Square(Inertia.M[1][0]) + FMath::Square(Inertia.M[2][0]) + FMath::Square(Inertia.M[2][1]);
		FRealDouble Trace = (Inertia.M[0][0] + Inertia.M[1][1] + Inertia.M[2][2]) / 3;

		if (Trace <= UE_SMALL_NUMBER)
		{
			// Tiny inertia - numerical instability would follow. We should not get this unless we have bad input.
			return TRotation<T,3>::FromIdentity();
		}

		// If the matrix is "almost diagonal" we are already in local space.
		// This is especially important when we built our inertia from a surface integral over a 
		// set of triangles because it will have accumulation errors that give non-zero off-diagonal elements
		// and we can end up selecting very unexpected axes for our eigenvectors.
		// This tolerance is pretty arbitrary - what value should we use?
		if ((OffDiagSizeSq / FMath::Square(Trace)) < UE_SMALL_NUMBER)
		{
			return TRotation<T,3>::FromIdentity();
		}

		T Size = static_cast<T>(FMath::Sqrt((FMath::Square(Inertia.M[0][0] - Trace) + FMath::Square(Inertia.M[1][1] - Trace) + FMath::Square(Inertia.M[2][2] - Trace) + 2. * OffDiagSizeSq) / 6.));
		PMatrix<T,3,3> NewMat = (Inertia - PMatrix<T,3,3>::Identity * static_cast<T>(Trace)) * static_cast<T>((1 / Size));
		T HalfDeterminant = NewMat.Determinant() / 2;
		T Angle = HalfDeterminant <= -1 ? UE_PI / 3 : (HalfDeterminant >= 1 ? 0 : FMath::Acos(HalfDeterminant) / 3);
		
		T m00 = static_cast<T>(Trace + 2 * Size * FMath::Cos(Angle));
		T m11 = static_cast<T>(Trace + 2 * Size * FMath::Cos(Angle + (2 * UE_PI / 3)));
		T m22 = static_cast<T>(3 * Trace - m00 - m11);

		// Extract Eigenvectors
		bool DoSwap = ((m00 - m11) > (m11 - m22)) ? false : true;
		TVec3<T> Eigenvector0 = (Inertia.SubtractDiagonal(DoSwap ? m22 : m00)).SymmetricCofactorMatrix().LargestColumnNormalized();
		TVec3<T> Orthogonal = Eigenvector0.GetOrthogonalVector().GetSafeNormal();
		
		PMatrix<T,3,2> Cofactors(Orthogonal, TVec3<T>::CrossProduct(Eigenvector0, Orthogonal));
		PMatrix<T,3,2> CofactorsScaled = Inertia * Cofactors;
		
		PMatrix<T,2,2> IR(
			CofactorsScaled.M[0] * Cofactors.M[0] + CofactorsScaled.M[1] * Cofactors.M[1] + CofactorsScaled.M[2] * Cofactors.M[2],
			CofactorsScaled.M[3] * Cofactors.M[0] + CofactorsScaled.M[4] * Cofactors.M[1] + CofactorsScaled.M[5] * Cofactors.M[2],
			CofactorsScaled.M[3] * Cofactors.M[3] + CofactorsScaled.M[4] * Cofactors.M[4] + CofactorsScaled.M[5] * Cofactors.M[5]);

		PMatrix<T,2,2> IM1 = IR.SubtractDiagonal(DoSwap ? m00 : m22);
		T OffDiag = IM1.M[1] * IM1.M[1];
		T IM1Scale0 = FMath::Max(T(0), IM1.M[3] * IM1.M[3] + OffDiag);
		T IM1Scale1 = FMath::Max(T(0), IM1.M[0] * IM1.M[0] + OffDiag);
		T SqrtIM1Scale0 = FMath::Sqrt(IM1Scale0);
		T SqrtIM1Scale1 = FMath::Sqrt(IM1Scale1);

		TVec3<T> Eigenvector2, Eigenvector1;
		if ((SqrtIM1Scale0 < UE_KINDA_SMALL_NUMBER) && (SqrtIM1Scale1 < UE_KINDA_SMALL_NUMBER))
		{
			Eigenvector1 = Orthogonal;
			Eigenvector2 = TVec3<T>::CrossProduct(Eigenvector0, Orthogonal).GetSafeNormal();
		}
		else
		{
			TVec2<T> SmallEigenvector2 = IM1Scale0 > IM1Scale1 
				? (TVec2<T>(IM1.M[3], -IM1.M[1]) / SqrtIM1Scale0) 
				: (IM1Scale1 > 0 ? (TVec2<T>(-IM1.M[1], IM1.M[0]) / SqrtIM1Scale1) : TVec2<T>(1, 0));
			Eigenvector2 = (Cofactors * SmallEigenvector2).GetSafeNormal();
			Eigenvector1 = TVec3<T>::CrossProduct(Eigenvector2, Eigenvector0).GetSafeNormal();
		}

		// Return results
		Inertia = PMatrix<T,3,3>(m00, 0, 0, m11, 0, m22);
		PMatrix<T,3,3> RotationMatrix = DoSwap ? PMatrix<T,3,3>(Eigenvector2, Eigenvector1, -Eigenvector0) : PMatrix<T,3,3>(Eigenvector0, Eigenvector1, Eigenvector2);

		// NOTE: UE Matrix are column-major, so the PMatrix constructor is not setting eigenvectors - we need to transpose it to get a UE rotation matrix.
		FinalRotation = TRotation<T,3>(RotationMatrix.GetTransposed());
		if (!ensure(FMath::IsNearlyEqual((float)FinalRotation.Size(), 1.0f, UE_KINDA_SMALL_NUMBER)))
		{
			return TRotation<T,3>::FromIdentity();
		}
		
		return FinalRotation;
	}

	void TransformToLocalSpace(FMassProperties& MassProperties)
	{
		// Diagonalize inertia
		const FRotation3 InertiaRotation = TransformToLocalSpace(MassProperties.InertiaTensor);

		// Calculate net rotation
		MassProperties.RotationOfMass = MassProperties.RotationOfMass * InertiaRotation;
	}


	void CalculateVolumeAndCenterOfMass(const FBox& BoundingBox, FVector::FReal& OutVolume, FVector& OutCenterOfMass)
	{
		const FVector Extents = static_cast<FVector::FReal>(2) * BoundingBox.GetExtent(); // FBox::GetExtent() returns half the size, but FAABB::Extents() returns total size
		OutVolume = Extents.X * Extents.Y * Extents.Z;
		OutCenterOfMass = BoundingBox.GetCenter();
	}

	void CalculateInertiaAndRotationOfMass(const FBox& BoundingBox, const FVector::FReal Density, FMatrix33& OutInertiaTensor, FRotation3& OutRotationOfMass)
	{
		const FVector Extents = static_cast<FVector::FReal>(2) * BoundingBox.GetExtent(); // FBox::GetExtent() returns half the size, but FAABB::Extents() returns total size
		const FVector::FReal Volume = Extents.X * Extents.Y * Extents.Z;
		const FVector::FReal Mass = Volume * Density;
		const FVector::FReal ExtentsYZ = Extents.Y * Extents.Y + Extents.Z * Extents.Z;
		const FVector::FReal ExtentsXZ = Extents.X * Extents.X + Extents.Z * Extents.Z;
		const FVector::FReal ExtentsXY = Extents.X * Extents.X + Extents.Y * Extents.Y;
		OutInertiaTensor = FMatrix33((Mass * ExtentsYZ) / 12., (Mass * ExtentsXZ) / 12., (Mass * ExtentsXY) / 12.);
		OutRotationOfMass = FRotation3::Identity;
	}
	
	FMassProperties CalculateMassProperties(const FBox& BoundingBox, const FVector::FReal Density)
	{
		FMassProperties MassProperties;
		CalculateVolumeAndCenterOfMass(BoundingBox, MassProperties.Volume, MassProperties.CenterOfMass);
		check(Density > 0);
		MassProperties.Mass = MassProperties.Volume * Density;
		CalculateInertiaAndRotationOfMass(BoundingBox, Density, MassProperties.InertiaTensor, MassProperties.RotationOfMass);
		return MassProperties;
	}
	
	template<typename T, typename TVec, typename TSurfaces>
	void CalculateVolumeAndCenterOfMassImpl(const TArray<TVec>& Vertices, const TSurfaces& Surfaces, T& OutVolume, TVec& OutCenterOfMass)
	{
		if (!Surfaces.Num())
		{
			OutVolume = 0;
			return;
		}
		
		double Volume = 0;
		FVec3 VolumeTimesSum(0);
		FVec3 Center = Vertices[Surfaces[0][0]];
		for (const auto& Element : Surfaces)
		{
			// implicitly triangulate with a fan
			for (int32 FanIdx = 0; FanIdx + 2 < Element.Num(); ++FanIdx)
			{
				int32 FanSubInds[]{ 0, FanIdx + 1, FanIdx + 2 };
				FMatrix33 DeltaMatrix;
				FVec3 PerElementSize;
				for (int32 i = 0; i < 3; ++i)
				{
					FVec3 DeltaVector = FVec3(Vertices[Element[FanSubInds[i]]]) - Center;
					DeltaMatrix.M[0][i] = DeltaVector[0];
					DeltaMatrix.M[1][i] = DeltaVector[1];
					DeltaMatrix.M[2][i] = DeltaVector[2];
				}

				PerElementSize[0] = DeltaMatrix.M[0][0] + DeltaMatrix.M[0][1] + DeltaMatrix.M[0][2];
				PerElementSize[1] = DeltaMatrix.M[1][0] + DeltaMatrix.M[1][1] + DeltaMatrix.M[1][2];
				PerElementSize[2] = DeltaMatrix.M[2][0] + DeltaMatrix.M[2][1] + DeltaMatrix.M[2][2];

				double Det = DeltaMatrix.M[0][0] * (DeltaMatrix.M[1][1] * DeltaMatrix.M[2][2] - DeltaMatrix.M[1][2] * DeltaMatrix.M[2][1]) -
					DeltaMatrix.M[0][1] * (DeltaMatrix.M[1][0] * DeltaMatrix.M[2][2] - DeltaMatrix.M[1][2] * DeltaMatrix.M[2][0]) +
					DeltaMatrix.M[0][2] * (DeltaMatrix.M[1][0] * DeltaMatrix.M[2][1] - DeltaMatrix.M[1][1] * DeltaMatrix.M[2][0]);

				Volume += Det;
				VolumeTimesSum += Det * PerElementSize;
			}
		}
		// @todo(mlentine): Should add support for thin shell mass properties
		if (Volume < UE_DOUBLE_KINDA_SMALL_NUMBER)	//handle negative volume using fallback for now. Need to investigate cases where this happens
			{
			OutVolume = 0;
			return;
			}
		OutCenterOfMass = TVec(Center + VolumeTimesSum / (4 * Volume));
		OutVolume = T(Volume / 6);
	}

	template<typename T, typename TSurfaces>
	void CHAOS_API CalculateVolumeAndCenterOfMass(const TParticles<T, 3>& Vertices, const TSurfaces& Surfaces, T& OutVolume, TVec3<T>& OutCenterOfMass)
	{
		CalculateVolumeAndCenterOfMassImpl(Vertices.AllX(), Surfaces, OutVolume, OutCenterOfMass);
	}

	template<typename T, typename TSurfaces>
	void CHAOS_API CalculateVolumeAndCenterOfMass(const TArray<TVec3<T>>& Vertices, const TSurfaces& Surfaces, T& OutVolume, TVec3<T>& OutCenterOfMass)
	{
		CalculateVolumeAndCenterOfMassImpl(Vertices, Surfaces, OutVolume, OutCenterOfMass);
	}

	template<typename T, typename TSurfaces>
	void CHAOS_API CalculateVolumeAndCenterOfMass(const TArray<UE::Math::TVector<T>>& Vertices, const TSurfaces& Surfaces, T& OutVolume, UE::Math::TVector<T>& OutCenterOfMass)
	{
		CalculateVolumeAndCenterOfMassImpl(Vertices, Surfaces, OutVolume, OutCenterOfMass);
	}

	template <typename T, typename TVec, typename TSurfaces>
	void CalculateInertiaAndRotationOfMassImpl(const TArray<TVec>& Vertices, const TSurfaces& Surfaces, const T Density, const TVec& CenterOfMass,
		PMatrix<T, 3, 3>& OutInertiaTensor, TRotation<T, 3>& OutRotationOfMass)
	{
		check(Density > 0);

		// Perform the calculation in doubles regardless of input/output type to reduce accumulation errors
		static const FMatrix33 Standard(2, 1, 1, 2, 1, 2);
		FMatrix33 Covariance(0);
		for (const auto& Element : Surfaces)
		{
			// implicitly triangulate with a fan
			for (int32 FanIdx = 0; FanIdx + 2 < Element.Num(); ++FanIdx)
			{
				int32 FanSubInds[]{ 0, FanIdx + 1, FanIdx + 2 };
				FMatrix33 DeltaMatrix(0);
				for (int32 i = 0; i < 3; ++i)
				{
					FVec3 DeltaVector = FVec3(Vertices[Element[FanSubInds[i]]] - CenterOfMass);
					DeltaMatrix.M[0][i] = DeltaVector[0];
					DeltaMatrix.M[1][i] = DeltaVector[1];
					DeltaMatrix.M[2][i] = DeltaVector[2];
				}
				const FReal Det = DeltaMatrix.M[0][0] * (DeltaMatrix.M[1][1] * DeltaMatrix.M[2][2] - DeltaMatrix.M[1][2] * DeltaMatrix.M[2][1]) -
					DeltaMatrix.M[0][1] * (DeltaMatrix.M[1][0] * DeltaMatrix.M[2][2] - DeltaMatrix.M[1][2] * DeltaMatrix.M[2][0]) +
					DeltaMatrix.M[0][2] * (DeltaMatrix.M[1][0] * DeltaMatrix.M[2][1] - DeltaMatrix.M[1][1] * DeltaMatrix.M[2][0]);
				const FMatrix33 ScaledStandard = Standard * Det;
				Covariance += DeltaMatrix * ScaledStandard * DeltaMatrix.GetTransposed();
			}
		}
		const FReal Trace = Covariance.M[0][0] + Covariance.M[1][1] + Covariance.M[2][2];
		const FMatrix33 TraceMat(Trace, Trace, Trace);
		FMatrix33 Inertia = (TraceMat - Covariance) * (Density / FReal(120));
		const FRotation3 Rotation = TransformToLocalSpace(Inertia);
		OutInertiaTensor = PMatrix<T, 3, 3>(Inertia);
		OutRotationOfMass = TRotation<T, 3>(Rotation);
	}

	template<typename T, typename TSurfaces>
	void CHAOS_API CalculateInertiaAndRotationOfMass(const TParticles<T, 3>& Vertices, const TSurfaces& Surfaces, const T Density, const TVec3<T>& CenterOfMass,
		PMatrix<T, 3, 3>& OutInertiaTensor, TRotation<T, 3>& OutRotationOfMass)
	{
		CalculateInertiaAndRotationOfMassImpl(Vertices.AllX(), Surfaces, Density, CenterOfMass, OutInertiaTensor, OutRotationOfMass);
	}

	template<typename T, typename TSurfaces>
	void CHAOS_API CalculateInertiaAndRotationOfMass(const TArray<UE::Math::TVector<T>>& Vertices, const TSurfaces& Surfaces, const T Density, const UE::Math::TVector<T>& CenterOfMass,
		PMatrix<T,3,3>& OutInertiaTensor, TRotation<T, 3>& OutRotationOfMass)
	{
		CalculateInertiaAndRotationOfMassImpl(Vertices, Surfaces, Density, CenterOfMass, OutInertiaTensor, OutRotationOfMass);
	}
	
	template<typename TSurfaces>
	FMassProperties CalculateMassProperties(
		const FParticles & Vertices,
		const TSurfaces& Surfaces,
		const FReal Mass)
	{
		FMassProperties MassProperties;
		CalculateVolumeAndCenterOfMass(Vertices, Surfaces, MassProperties.Volume, MassProperties.CenterOfMass);

		check(Mass > 0);
		check(MassProperties.Volume > UE_SMALL_NUMBER);
		CalculateInertiaAndRotationOfMass(Vertices, Surfaces, Mass / MassProperties.Volume, MassProperties.CenterOfMass, MassProperties.InertiaTensor, MassProperties.RotationOfMass);
		
		return MassProperties;
	}

	FMassProperties Combine(const TArray<FMassProperties>& MPArray)
	{
		FMassProperties NewMP = CombineWorldSpace(MPArray);
		TransformToLocalSpace(NewMP);
		return NewMP;
	}

	FMassProperties CombineWorldSpace(const TArray<FMassProperties>& MPArray)
	{
		check(MPArray.Num() > 0);

		if ((MPArray.Num() == 1) && MPArray[0].RotationOfMass.IsIdentity())
		{
			return MPArray[0];
		}

		FMassProperties NewMP;

		for (const FMassProperties& Child : MPArray)
		{
			NewMP.Volume += Child.Volume;
			NewMP.InertiaTensor += Utilities::ComputeWorldSpaceInertia(Child.RotationOfMass, Child.InertiaTensor);
			NewMP.CenterOfMass += Child.CenterOfMass * Child.Mass;
			NewMP.Mass += Child.Mass;
		}

		// Default to 100cm cube of water for zero mass and volume objects
		if (!ensureMsgf(NewMP.Mass > UE_SMALL_NUMBER, TEXT("CombineWorldSpace: zero total mass detected")))
		{
			const FReal Dim = 100;	// cm
			const FReal Density = FReal(0.001); // kg/cm3
			NewMP.Volume = Dim * Dim * Dim;
			NewMP.Mass = NewMP.Volume * Density;
			NewMP.InertiaTensor = (NewMP.Mass * Dim * Dim / FReal(6)) * FMatrix33::Identity;
			NewMP.CenterOfMass = FVec3(0);
			return NewMP;
		}

		NewMP.CenterOfMass /= NewMP.Mass;

		if (MPArray.Num() > 1)
		{
			for (const FMassProperties& Child : MPArray)
			{
				const FReal M = Child.Mass;
				const FVec3 ParentToChild = Child.CenterOfMass - NewMP.CenterOfMass;
				const FReal P0 = ParentToChild[0];
				const FReal P1 = ParentToChild[1];
				const FReal P2 = ParentToChild[2];
				const FReal MP0P0 = M * P0 * P0;
				const FReal MP1P1 = M * P1 * P1;
				const FReal MP2P2 = M * P2 * P2;
				NewMP.InertiaTensor += FMatrix33(MP1P1 + MP2P2, -M * P1 * P0, -M * P2 * P0, MP2P2 + MP0P0, -M * P2 * P1, MP1P1 + MP0P0);
			}
		}

		return NewMP;
	}

	bool CalculateMassPropertiesOfImplicitUnion(
		Chaos::FMassProperties& OutMassProperties,
		const Chaos::FRigidTransform3& WorldTransform,
		const Chaos::FImplicitObjectUnion& ImplicitUnion,
		Chaos::FReal InDensityKGPerCM)
	{
		Chaos::FReal TotalMass = 0;
		Chaos::FReal TotalVolume = 0;
		Chaos::FVec3 TotalCenterOfMass(0);
		TArray< Chaos::FMassProperties > MassPropertiesList;
		for (const Chaos::FImplicitObjectPtr& ImplicitObjectUniquePtr: ImplicitUnion.GetObjects())
		{
			if (const Chaos::FImplicitObject* ImplicitObject = ImplicitObjectUniquePtr.GetReference())
			{
				Chaos::FMassProperties MassProperties;
				if (CalculateMassPropertiesOfImplicitType(MassProperties, FTransform::Identity, ImplicitObject, InDensityKGPerCM))
				{
					MassPropertiesList.Add(MassProperties);
					TotalMass += MassProperties.Mass;
					TotalVolume += MassProperties.Volume;
					TotalCenterOfMass += MassProperties.CenterOfMass * MassProperties.Mass;
				}
			}
		}

		Chaos::FMatrix33 Tensor;

		// If no shapes contribute to mass, or they are scaled to zero, we may end up with zero mass here
		if ((TotalMass > 0.f) && (MassPropertiesList.Num() > 0))
		{
			TotalCenterOfMass /= TotalMass;

			const Chaos::FMassProperties CombinedMassProperties = Chaos::CombineWorldSpace(MassPropertiesList);
			ensure(CombinedMassProperties.RotationOfMass.IsIdentity());
			Tensor = CombinedMassProperties.InertiaTensor;
		}
		else
		{
			// @todo(chaos): We should support shape-less particles as long as their mass an inertia are set directly
			// For now hard-code a 50cm sphere with density 1g/cc
			Tensor = Chaos::FMatrix33(5.24e5f, 5.24e5f, 5.24e5f);
			TotalMass = 523.0f;
			TotalVolume = 523000;
		}

		OutMassProperties.InertiaTensor = Tensor;
		OutMassProperties.Mass = TotalMass;
		OutMassProperties.Volume = TotalVolume;
		OutMassProperties.CenterOfMass = TotalCenterOfMass;
		OutMassProperties.RotationOfMass = Chaos::FRotation3::Identity;

		return (OutMassProperties.Mass > 0);
	}
	
	bool CalculateMassPropertiesOfImplicitType(
		Chaos::FMassProperties& OutMassProperties,
		const Chaos::FRigidTransform3& WorldTransform,
		const Chaos::FImplicitObject* ImplicitObject,
		Chaos::FReal InDensityKGPerCM)
	{
		using namespace Chaos;

		if (ImplicitObject)
		{
			// Hack to handle Transformed and Scaled<ImplicitObjectTriangleMesh> until CastHelper can properly support transformed
			// Commenting this out temporarily as it breaks vehicles
			/*	if (Chaos::IsScaled(ImplicitObject->GetType(true)) && Chaos::GetInnerType(ImplicitObject->GetType(true)) & Chaos::ImplicitObjectType::TriangleMesh)
				{
					OutMassProperties.Volume = 0.f;
					OutMassProperties.Mass = FLT_MAX;
					OutMassProperties.InertiaTensor = FMatrix33(0, 0, 0);
					OutMassProperties.CenterOfMass = FVector(0);
					OutMassProperties.RotationOfMass = Chaos::FRotation3::FromIdentity();
					return false;
				}
				else if (ImplicitObject->GetType(true) & Chaos::ImplicitObjectType::TriangleMesh)
				{
					OutMassProperties.Volume = 0.f;
					OutMassProperties.Mass = FLT_MAX;
					OutMassProperties.InertiaTensor = FMatrix33(0, 0, 0);
					OutMassProperties.CenterOfMass = FVector(0);
					OutMassProperties.RotationOfMass = Chaos::FRotation3::FromIdentity();
					return false;
				}
			else*/
			if (ImplicitObject->IsUnderlyingUnion())
			{
				const FImplicitObjectUnion* Union = static_cast<const FImplicitObjectUnion*>(ImplicitObject);
				CalculateMassPropertiesOfImplicitUnion(OutMassProperties, WorldTransform, *Union, InDensityKGPerCM);
			}
			else
			{
				Chaos::Utilities::CastHelper(*ImplicitObject, FTransform::Identity, [&OutMassProperties, InDensityKGPerCM](const auto& Object, const auto& LocalTM)
					{
						OutMassProperties.Volume = Object.GetVolume();
						OutMassProperties.Mass = OutMassProperties.Volume * InDensityKGPerCM;
						OutMassProperties.InertiaTensor = Object.GetInertiaTensor(OutMassProperties.Mass);
						OutMassProperties.CenterOfMass = LocalTM.TransformPosition(Object.GetCenterOfMass());
						OutMassProperties.RotationOfMass = LocalTM.GetRotation() * Object.GetRotationOfMass();
					});
			}
		}

		// If the implicit is null, or it is scaled to zero it will have zero volume, mass or inertia
		return (OutMassProperties.Mass > 0);
	}

	void CalculateMassPropertiesFromShapeCollection(
		Chaos::FMassProperties& OutProperties, 
		int32 InNumShapes, 
		Chaos::FReal InDensityKGPerCM,
		const TArray<bool>& bContributesToMass,
		TFunction<Chaos::FPerShapeData* (int32 ShapeIndex)> GetShapeDelegate)
	{
		Chaos::FReal TotalMass = 0;
		Chaos::FReal TotalVolume = 0;
		Chaos::FVec3 TotalCenterOfMass(0);
		TArray< Chaos::FMassProperties > MassPropertiesList;
		for (int32 ShapeIndex = 0; ShapeIndex < InNumShapes; ++ShapeIndex)
		{
			const Chaos::FPerShapeData* Shape = GetShapeDelegate(ShapeIndex);

			const bool bHassMass = (ShapeIndex < bContributesToMass.Num()) ? bContributesToMass[ShapeIndex] : true;
			if (bHassMass)
			{
				if (const Chaos::FImplicitObject* ImplicitObject = Shape->GetGeometry())
				{
					Chaos::FMassProperties MassProperties;
					if (CalculateMassPropertiesOfImplicitType(MassProperties, FTransform::Identity, ImplicitObject, InDensityKGPerCM))
					{
						MassPropertiesList.Add(MassProperties);
						TotalMass += MassProperties.Mass;
						TotalVolume += MassProperties.Volume;
						TotalCenterOfMass += MassProperties.CenterOfMass * MassProperties.Mass;
					}
				}
			}
		}

		Chaos::FMatrix33 Tensor;

		// If no shapes contribute to mass, or they are scaled to zero, we may end up with zero mass here
		if ((TotalMass > 0.f) && (MassPropertiesList.Num() > 0))
		{
			TotalCenterOfMass /= TotalMass;

			Chaos::FMassProperties CombinedMassProperties = Chaos::CombineWorldSpace(MassPropertiesList);
			ensure(CombinedMassProperties.RotationOfMass.IsIdentity());
			Tensor = CombinedMassProperties.InertiaTensor;
		}
		else
		{
			// @todo(chaos): We should support shape-less particles as long as their mass an inertia are set directly
			// For now hard-code a 50cm sphere with density 1g/cc
			Tensor = Chaos::FMatrix33(5.24e5f, 5.24e5f, 5.24e5f);
			TotalMass = 523.0f;
			TotalVolume = 523000;
		}

		OutProperties.InertiaTensor = Tensor;
		OutProperties.Mass = TotalMass;
		OutProperties.Volume = TotalVolume;
		OutProperties.CenterOfMass = TotalCenterOfMass;
		OutProperties.RotationOfMass = Chaos::FRotation3::Identity;
	}
	
	template CHAOS_API TRotation<FRealSingle,3> TransformToLocalSpace(PMatrix<FRealSingle,3,3>& Inertia);
	template CHAOS_API TRotation<FRealDouble,3> TransformToLocalSpace(PMatrix<FRealDouble,3,3>& Inertia);
	
	template CHAOS_API FMassProperties CalculateMassProperties(const FParticles& Vertices, const TArray<TVec3<int32>>& Surfaces, const FReal Mass);
	template CHAOS_API FMassProperties CalculateMassProperties(const FParticles & Vertices, const TArray<TArray<int32>>& Surfaces, const FReal Mass);

	template CHAOS_API void CalculateVolumeAndCenterOfMass(const TParticles<FRealDouble, 3>& Vertices, const TArray<TVec3<int32>>& Surfaces, FRealDouble& OutVolume, TVec3<FRealDouble>& OutCenterOfMass);
	template CHAOS_API void CalculateVolumeAndCenterOfMass(const TParticles<FRealDouble, 3>& Vertices, const TArray<TArray<int32>>& Surfaces, FRealDouble& OutVolume, TVec3<FRealDouble>& OutCenterOfMass);

	template CHAOS_API void CalculateVolumeAndCenterOfMass(const TParticles<FRealSingle, 3>& Vertices, const TArray<TVec3<int32>>& Surfaces, FRealSingle& OutVolume, TVec3<FRealSingle>& OutCenterOfMass);
	template CHAOS_API void CalculateVolumeAndCenterOfMass(const TParticles<FRealSingle, 3>& Vertices, const TArray<TArray<int32>>& Surfaces, FRealSingle& OutVolume, TVec3<FRealSingle>& OutCenterOfMass);
	
	template CHAOS_API void CalculateVolumeAndCenterOfMass(const TArray<TVec3<FRealSingle>>& Vertices, const TArray<TArray<int32>>& Surfaces, FRealSingle& OutVolume, TVec3<FRealSingle>& OutCenterOfMass);
	template CHAOS_API void CalculateVolumeAndCenterOfMass(const TArray<TVec3<FRealDouble>>& Vertices, const TArray<TArray<int32>>& Surfaces, FRealDouble& OutVolume, TVec3<FRealDouble>& OutCenterOfMass);

	template CHAOS_API void CalculateVolumeAndCenterOfMass(const TArray<UE::Math::TVector<float>>& Vertices, const TArray<TVec3<int32>>& Surfaces, float& OutVolume, UE::Math::TVector<float>& OutCenterOfMass);

	template void CHAOS_API CalculateInertiaAndRotationOfMass(const TParticles<FRealSingle,3>& Vertices, const TArray<TVec3<int32>>& Surfaces, const FRealSingle Density, const TVec3<FRealSingle>& CenterOfMass,
		PMatrix<FRealSingle,3,3>& OutInertiaTensor, TRotation<FRealSingle, 3>& OutRotationOfMass);
	template void CHAOS_API CalculateInertiaAndRotationOfMass(const TParticles<FRealSingle,3>& Vertices, const TArray<TArray<int32>>& Surfaces, const FRealSingle Density, const TVec3<FRealSingle>& CenterOfMass,
		PMatrix<FRealSingle,3,3>& OutInertiaTensor, TRotation<FRealSingle, 3>& OutRotationOfMass);

	template void CHAOS_API CalculateInertiaAndRotationOfMass(const TParticles<FRealDouble,3>& Vertices, const TArray<TVec3<int32>>& Surfaces, const FRealDouble Density, const TVec3<FRealDouble>& CenterOfMass,
		PMatrix<FRealDouble,3,3>& OutInertiaTensor, TRotation<FRealDouble, 3>& OutRotationOfMass);
	template void CHAOS_API CalculateInertiaAndRotationOfMass(const TParticles<FRealDouble,3>& Vertices, const TArray<TArray<int32>>& Surfaces, const FRealDouble Density, const TVec3<FRealDouble>& CenterOfMass,
		PMatrix<FRealDouble,3,3>& OutInertiaTensor, TRotation<FRealDouble, 3>& OutRotationOfMass);

	template void CHAOS_API CalculateInertiaAndRotationOfMass(const TArray<UE::Math::TVector<float>>& Vertices, const TArray<TVec3<int32>>& Surfaces, const float Density, const UE::Math::TVector<float>& CenterOfMass,
		PMatrix<float,3,3>& OutInertiaTensor, TRotation<float, 3>& OutRotationOfMass);
	
}
