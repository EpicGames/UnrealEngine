// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/Math/BlockSparseLinearSystem.h"
#include "ChaosCheck.h"
#include "ChaosLog.h"
#include "Chaos/Framework/Parallel.h"
#include "GenericPlatform/GenericPlatformMath.h"
#include "Misc/AssertionMacros.h"
#include "Chaos/Vector.h"
#include "HAL/IConsoleManager.h"

#if defined(_MSC_VER) && USING_CODE_ANALYSIS
#pragma warning(push)
#pragma warning(disable : 6011)
#pragma warning(disable : 6387)
#pragma warning(disable : 6313)
#pragma warning(disable : 6294)
#endif
PRAGMA_DEFAULT_VISIBILITY_START
THIRD_PARTY_INCLUDES_START
#include <Eigen/Sparse>
#include <Eigen/Core>
#include <Eigen/Dense>
THIRD_PARTY_INCLUDES_END
PRAGMA_DEFAULT_VISIBILITY_END
#if defined(_MSC_VER) && USING_CODE_ANALYSIS
#pragma warning(pop)
#endif


namespace Chaos {
enum EPreconditionerType
{
	Diagonal = 0,
	IncompleteCholesky = 1
};
static int32 PreconditionerType = EPreconditionerType::Diagonal;
FAutoConsoleVariableRef CVarChaosLinearSystemPreconditioner(TEXT("p.Chaos.LinearSystem.Preconditioner"), PreconditionerType, TEXT("0 = Diagonal, 1 = IncompleteCholesky"));

	template<typename T, int32 BlockSize>
	struct TBlockSparseSymmetricLinearSystem<T, BlockSize>::FPimpl
	{
		// Just doing Eigen sparse matrix for now and building from triplets. Will investigate block symmetric later
		typedef typename Eigen::SparseMatrix<T, Eigen::RowMajor> SparseMatrixType;
		typedef typename Eigen::Triplet<T> MatrixTripletType;
		typedef typename Eigen::Matrix<T, Eigen::Dynamic, 1> RealVectorType;
		typedef typename Eigen::Map<RealVectorType> VectorType;
		typedef typename Eigen::Map<const RealVectorType> ConstVectorType;

		SparseMatrixType AMatrix;

		// TripletBuilderNum is the actual number of entries.
		// The TArray Num may be larger as it represents the amount of reserved space.
		TArray<MatrixTripletType> MatrixBuilderTriplets;
		std::atomic<int32> TripletBuilderNum;

		FPimpl()
			: TripletBuilderNum(0)
		{}

		void Reset(int32 NumRows)
		{
			TripletBuilderNum.store(0);
			MatrixBuilderTriplets.Reset();
			AMatrix.resize(NumRows*BlockSize, NumRows*BlockSize);
		}
		
		void ReserveForParallelAdd(int32 NumDiagEntries, int32 NumOffDiagEntries)
		{
			MatrixBuilderTriplets.SetNum(TripletBuilderNum.load() + BlockSize * BlockSize* (NumDiagEntries + NumOffDiagEntries * 2));
		}

		void AddMatrixEntry(int32 Index0, int32 Index1, const Chaos::PMatrix<T, BlockSize, BlockSize>& AEntry)
		{
			if (Index0 == Index1)
			{
				int32 BuilderIndex = TripletBuilderNum.fetch_add(9);
				for (int32 I = 0; I < BlockSize; ++I)
				{
					for (int32 J = 0; J < BlockSize; ++J)
					{
						check(FMath::IsNearlyEqual(AEntry.M[I][J], AEntry.M[J][I]));
						check(BlockSize * Index0 + I < AMatrix.rows());
						check(BlockSize * Index1 + J < AMatrix.cols());
						MatrixBuilderTriplets[BuilderIndex++] = MatrixTripletType(BlockSize * Index0 + I, BlockSize * Index1 + J, AEntry.M[I][J]);
					}
				}
			}
			else
			{
				// Need to add symmetric part
				int32 BuilderIndex = TripletBuilderNum.fetch_add(18);
				for (int32 I = 0; I < BlockSize; ++I)
				{
					for (int32 J = 0; J < BlockSize; ++J)
					{
						check(BlockSize * Index0 + I < AMatrix.rows());
						check(BlockSize * Index1 + J < AMatrix.cols());
						check(BlockSize * Index0 + I < AMatrix.cols());
						check(BlockSize * Index1 + J < AMatrix.rows());
						MatrixBuilderTriplets[BuilderIndex++] = MatrixTripletType(BlockSize * Index0 + I, BlockSize * Index1 + J, AEntry.M[I][J]);
						MatrixBuilderTriplets[BuilderIndex++] = MatrixTripletType(BlockSize * Index1 + J, BlockSize * Index0 + I, AEntry.M[I][J]);
					}
				}
			}
		}

		void BuildMatrix()
		{
			MatrixBuilderTriplets.SetNum(TripletBuilderNum.load(), EAllowShrinking::No);
			TRACE_CPUPROFILER_EVENT_SCOPE(ChaosBlockSparseLinearSystem_BuildMatrix);
			AMatrix.setFromTriplets(MatrixBuilderTriplets.GetData(), MatrixBuilderTriplets.GetData() + MatrixBuilderTriplets.Num());
		}

		template<typename Preconditioner>
		bool Solve(const TConstArrayView<TVector<T, BlockSize>>& RHS, const TArrayView<TVector<T, BlockSize>>& Result, const int32 MaxNumCGIterations, const T CGResidualTolerance, bool bCheckResidual, int32* OutIterations, T* OutError)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ChaosBlockSparseLinearSystem_CGSolve);
			check(Result.Num() == RHS.Num());
			ConstVectorType WrappedRHS((const T*)RHS.GetData(), RHS.Num() * BlockSize);
			VectorType WrappedResult((T*)Result.GetData(), Result.Num() * BlockSize);

			// Just use Eigen's CG for now.
			Eigen::ConjugateGradient<SparseMatrixType, Eigen::Lower | Eigen::Upper, Preconditioner> CG;
			CG.setMaxIterations(MaxNumCGIterations);
			CG.setTolerance(CGResidualTolerance);
			// bCheckResidual is unused... Eigen always uses it
			CG.compute(AMatrix);
			WrappedResult = CG.solve(WrappedRHS);

			if (OutIterations)
			{
				*OutIterations = (int32)CG.iterations();
			}
			if (OutError)
			{
				*OutError = (T)CG.error();
			}
			return CG.info() == Eigen::ComputationInfo::Success;
		}
	};

	template<typename T, int32 BlockSize>
	TBlockSparseSymmetricLinearSystem<T, BlockSize>::TBlockSparseSymmetricLinearSystem()
		:Pimpl(MakePimpl<FPimpl>())
	{}
	template<typename T, int32 BlockSize>
	TBlockSparseSymmetricLinearSystem<T, BlockSize>::TBlockSparseSymmetricLinearSystem(TBlockSparseSymmetricLinearSystem&& Other)
		: Pimpl(MoveTemp(Other.Pimpl))
	{}
	template<typename T, int32 BlockSize>
	TBlockSparseSymmetricLinearSystem<T,BlockSize>& TBlockSparseSymmetricLinearSystem<T, BlockSize>::operator=(TBlockSparseSymmetricLinearSystem&& Other)
	{
		Pimpl = MoveTemp(Other.Pimpl);
		return *this;
	}

	template<typename T, int32 BlockSize>
	void TBlockSparseSymmetricLinearSystem<T, BlockSize>::Reset(int32 NumRows)
	{
		Pimpl->Reset(NumRows);
	}

	template<typename T, int32 BlockSize>
	void TBlockSparseSymmetricLinearSystem<T, BlockSize>::ReserveForParallelAdd(int32 NumDiagEntries, int32 NumOffDiagEntries)
	{
		Pimpl->ReserveForParallelAdd(NumDiagEntries, NumOffDiagEntries);
	}

	template<typename T, int32 BlockSize>
	void TBlockSparseSymmetricLinearSystem<T, BlockSize>::AddMatrixEntry(int32 Index0, int32 Index1, const Chaos::PMatrix<T, BlockSize, BlockSize>& AEntry)
	{
		Pimpl->AddMatrixEntry(Index0, Index1, AEntry);
	}

	template<typename T, int32 BlockSize>
	void TBlockSparseSymmetricLinearSystem<T, BlockSize>::FinalizeSystem()
	{
		Pimpl->BuildMatrix();
	}

	template<typename T, int32 BlockSize>
	bool TBlockSparseSymmetricLinearSystem<T, BlockSize>::Solve(const TConstArrayView<TVector<T, BlockSize>>& RHS, const TArrayView<TVector<T, BlockSize>>& Result,
		const int32 MaxNumCGIterations, const T CGResidualTolerance, bool bCheckResidual, 
		int32* OptionalOutIterations, T* OptionalOutError) const
	{
		switch (PreconditionerType)
		{
		case EPreconditionerType::Diagonal:
			return Pimpl->template Solve<Eigen::DiagonalPreconditioner<T>>(RHS, Result, MaxNumCGIterations, CGResidualTolerance, bCheckResidual, OptionalOutIterations, OptionalOutError);
		case EPreconditionerType::IncompleteCholesky:
			return Pimpl->template Solve<Eigen::IncompleteCholesky<T, Eigen::Upper | Eigen::Lower>>(RHS, Result, MaxNumCGIterations, CGResidualTolerance, bCheckResidual, OptionalOutIterations, OptionalOutError);
		default:
			checkNoEntry();
		}
		return false;
	}
	template class TBlockSparseSymmetricLinearSystem<FRealSingle, 3>;
	template class TBlockSparseSymmetricLinearSystem<FRealDouble, 3>;
}
