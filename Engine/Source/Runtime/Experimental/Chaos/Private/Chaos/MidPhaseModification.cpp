// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/MidPhaseModification.h"
#include "Chaos/Collision/CollisionConstraintAllocator.h"
#include "Chaos/Collision/PBDCollisionConstraint.h"
#include "Chaos/Collision/ParticlePairMidPhase.h"
#include "Chaos/Particle/ParticleUtilities.h"
#include "Chaos/PBDCollisionConstraints.h"
#include "Chaos/PBDRigidsEvolutionGBF.h"
#include "Chaos/ParticleHandleFwd.h"

namespace Chaos
{
	void FMidPhaseModifier::Disable()
	{
		MidPhase->SetIsActive(false);
	}

	void FMidPhaseModifier::DisableCCD()
	{
		MidPhase->SetCCDIsActive(false);
	}

	void FMidPhaseModifier::DisableConvexOptimization()
	{
		// SetConvexOptimizationIsActive may destroy constraints. If it does, we need to remove
		// them from the constraint graph first.
		// NOTE: We only destroy collisions if convex optimization use was changed. Otherwise
		// we would end up unnecessarily removing collisions every frame if DisableConvexOptimization()
		// is called repreatedly.
		const auto& RemoveCollisionFromGraph = [this](FPBDCollisionConstraint* InConstraint) -> ECollisionVisitorResult
			{
				if ((Accessor != nullptr) && (InConstraint != nullptr))
				{
					Accessor->GetEvolution().RemoveConstraintFromConstraintGraph(InConstraint->GetConstraintHandle());
				}
				return ECollisionVisitorResult::Continue;
			};

		if (MidPhase != nullptr)
		{
			MidPhase->SetConvexOptimizationIsActive(false, RemoveCollisionFromGraph);
		}
	}

	void FMidPhaseModifier::GetParticles(
		FGeometryParticleHandle** Particle0,
		FGeometryParticleHandle** Particle1) const
	{
		*Particle0 = MidPhase->GetParticle0();
		*Particle1 = MidPhase->GetParticle1();
	}

	FGeometryParticleHandle* FMidPhaseModifier::GetOtherParticle(const FGeometryParticleHandle* InParticle) 
	{
		if (MidPhase)
		{
			FGeometryParticleHandle* Particle0 = MidPhase->GetParticle0();
			FGeometryParticleHandle* Particle1 = MidPhase->GetParticle1();
			if (InParticle == Particle0)
			{
				return Particle1;
			}
			else if (InParticle == Particle1)
			{
				return Particle0;
			}
		}
		return nullptr;
	}

	const FGeometryParticleHandle* FMidPhaseModifier::GetOtherParticle(const FGeometryParticleHandle* InParticle) const
	{
		if (MidPhase)
		{
			const FGeometryParticleHandle* Particle0 = MidPhase->GetParticle0();
			const FGeometryParticleHandle* Particle1 = MidPhase->GetParticle1();
			if (InParticle == Particle0)
			{
				return Particle1;
			}
			else if (InParticle == Particle1)
			{
				return Particle0;
			}
		}
		return nullptr;
	}

	FMidPhaseModifierParticleIterator FMidPhaseModifierParticleRange::begin() const
	{
		return FMidPhaseModifierParticleIterator(Accessor, Particle);
	}

	FMidPhaseModifierParticleIterator FMidPhaseModifierParticleRange::end() const
	{
		return FMidPhaseModifierParticleIterator(Accessor, Particle, Particle->ParticleCollisions().Num());
	}

	int32 FMidPhaseModifierAccessor::GetNumMidPhases() const
	{
		return Evolution.GetCollisionConstraints().GetConstraintAllocator().GetNumMidPhases();
	}

	int32 FMidPhaseModifierAccessor::GetNumMidPhases(const FGeometryParticleHandle* Particle) const
	{
		return Particle->ParticleCollisions().Num();
	}

	FMidPhaseModifierParticleRange FMidPhaseModifierAccessor::GetMidPhases(FGeometryParticleHandle* Particle)
	{
		return FMidPhaseModifierParticleRange(this, Particle);
	}

	FMidPhaseModifier FMidPhaseModifierAccessor::GetMidPhase(
		FGeometryParticleHandle* Particle0,
		FGeometryParticleHandle* Particle1)
	{
		if (Particle0 && Particle1)
		{
			// Put the particle with fewer collisions in spot 0
			int32 Num0 = Particle0->ParticleCollisions().Num();
			int32 Num1 = Particle1->ParticleCollisions().Num();
			if (Num1 < Num0)
			{
				Swap(Particle0, Particle1);
			}

			// Loop over the mid-phases of the particle that has fewer of them,
			// find the one that involves the other particle, if any.
			for (FMidPhaseModifier& Modifier : GetMidPhases(Particle0))
			{
				if (Modifier.GetOtherParticle(Particle0) == Particle1)
				{
					return Modifier;
				}
			}
		}

		return FMidPhaseModifier();
	}

	void FMidPhaseModifierAccessor::VisitMidPhases(const TFunction<void(FMidPhaseModifier&)>& Visitor)
	{
		// This visitor is a thin function wrapper so that users can't directly access
		// the midphase itself, but instead can access a modifier.
		Evolution.GetCollisionConstraints().GetConstraintAllocator().VisitMidPhases([this, &Visitor](FParticlePairMidPhase& MidPhase)
		{
			FMidPhaseModifier MidPhaseModifier(&MidPhase, this);
			Visitor(MidPhaseModifier);
			return ECollisionVisitorResult::Continue;
		});
	}
}
