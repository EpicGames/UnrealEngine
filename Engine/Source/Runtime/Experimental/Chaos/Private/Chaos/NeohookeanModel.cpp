// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/NeohookeanModel.h"
#include "Chaos/ImplicitQRSVD.h"



namespace Chaos::Softs
{

template <typename T>
T PsiNeohookeanMM(const Chaos::PMatrix<T, 3, 3>& F, const T mu, const T lambda)
{
	T J = F.Determinant();
	Chaos::PMatrix<T, 3, 3> FTF = F* F.GetTransposed();
	T lambda_hat = mu + lambda;
	T gamma = T(1) + mu / lambda_hat;
	return T(0.5) * mu * (FTF.GetAt(0,0) + FTF.GetAt(1,1) + FTF.GetAt(2,2) - 3) + T(0.5) * lambda_hat * (J - gamma) * (J - gamma);
}


template <typename T>
void PNeohookeanMM(const Chaos::PMatrix<T, 3, 3>& Fe, const T mu, const T lambda, Chaos::PMatrix<T, 3, 3>& P)
{
	T J = Fe.Determinant();
	T lambda_hat = mu + lambda;
	Chaos::PMatrix<T, 3, 3> JFinvT((T)0.);
	JFinvT.SetAt(0, 0, Fe.GetAt(1, 1) * Fe.GetAt(2, 2) - Fe.GetAt(2, 1) * Fe.GetAt(1, 2));
	JFinvT.SetAt(0, 1, Fe.GetAt(2, 0) * Fe.GetAt(1, 2) - Fe.GetAt(1, 0) * Fe.GetAt(2, 2));
	JFinvT.SetAt(0, 2, Fe.GetAt(1, 0) * Fe.GetAt(2, 1) - Fe.GetAt(2, 0) * Fe.GetAt(1, 1));
	JFinvT.SetAt(1, 0, Fe.GetAt(2, 1) * Fe.GetAt(0, 2) - Fe.GetAt(0, 1) * Fe.GetAt(2, 2));
	JFinvT.SetAt(1, 1, Fe.GetAt(0, 0) * Fe.GetAt(2, 2) - Fe.GetAt(2, 0) * Fe.GetAt(0, 2));
	JFinvT.SetAt(1, 2, Fe.GetAt(2, 0) * Fe.GetAt(0, 1) - Fe.GetAt(0, 0) * Fe.GetAt(2, 1));
	JFinvT.SetAt(2, 0, Fe.GetAt(0, 1) * Fe.GetAt(1, 2) - Fe.GetAt(1, 1) * Fe.GetAt(0, 2));
	JFinvT.SetAt(2, 1, Fe.GetAt(1, 0) * Fe.GetAt(0, 2) - Fe.GetAt(0, 0) * Fe.GetAt(1, 2));
	JFinvT.SetAt(2, 2, Fe.GetAt(0, 0) * Fe.GetAt(1, 1) - Fe.GetAt(1, 0) * Fe.GetAt(0, 1));
	T gamma = T(1) + mu / lambda_hat;
	P = T(mu) * Fe+T(lambda_hat) * (J - gamma) * JFinvT;
}

}

template CHAOS_API void Chaos::Softs::PNeohookeanMM<Chaos::FReal>(const Chaos::PMatrix<Chaos::FReal, 3, 3>& Fe, const Chaos::FReal mu, const Chaos::FReal lambda, Chaos::PMatrix<Chaos::FReal, 3, 3>& P);
template CHAOS_API void Chaos::Softs::PNeohookeanMM<Chaos::FRealSingle>(const Chaos::PMatrix<Chaos::FRealSingle, 3, 3>& Fe, const Chaos::FRealSingle mu, const Chaos::FRealSingle lambda, Chaos::PMatrix<Chaos::FRealSingle, 3, 3>& P);
template CHAOS_API Chaos::FReal Chaos::Softs::PsiNeohookeanMM<Chaos::FReal>(const Chaos::PMatrix<Chaos::FReal, 3, 3>& F, const Chaos::FReal mu, const Chaos::FReal lambda);
template CHAOS_API Chaos::FRealSingle Chaos::Softs::PsiNeohookeanMM<Chaos::FRealSingle>(const Chaos::PMatrix<Chaos::FRealSingle, 3, 3>& F, const Chaos::FRealSingle mu, const Chaos::FRealSingle lambda);
