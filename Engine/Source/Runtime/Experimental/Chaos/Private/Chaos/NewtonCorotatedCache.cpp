// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/NewtonCorotatedCache.h"
#include "Chaos/ImplicitQRSVD.h"



namespace Chaos::Softs
{

template<typename T>
void CorotatedCache<T>::UpdateCache(const Chaos::PMatrix<T, 3, 3>& F) 
{
	Chaos::PMatrix<T, 3, 3> R((T)0.), S((T)0.), JFinvT((T)0.);
	Chaos::PolarDecomposition(F, R, S);

	JFInvTCache[0] = F.GetAt(1, 1) * F.GetAt(2, 2) - F.GetAt(2, 1) * F.GetAt(1, 2); JFInvTCache[1] = F.GetAt(2, 0) * F.GetAt(1, 2) - F.GetAt(1, 0) * F.GetAt(2, 2);  
	JFInvTCache[2] = F.GetAt(1, 0) * F.GetAt(2, 1) - F.GetAt(2, 0) * F.GetAt(1, 1); JFInvTCache[3] = F.GetAt(2, 1) * F.GetAt(0, 2) - F.GetAt(0, 1) * F.GetAt(2, 2);
	JFInvTCache[4] = F.GetAt(0, 0) * F.GetAt(2, 2) - F.GetAt(2, 0) * F.GetAt(0, 2); JFInvTCache[5] = F.GetAt(2, 0) * F.GetAt(0, 1) - F.GetAt(0, 0) * F.GetAt(2, 1);
	JFInvTCache[6] = F.GetAt(0, 1) * F.GetAt(1, 2) - F.GetAt(1, 1) * F.GetAt(0, 2); JFInvTCache[7] = F.GetAt(1, 0) * F.GetAt(0, 2) - F.GetAt(0, 0) * F.GetAt(1, 2);
	JFInvTCache[8] = F.GetAt(0, 0)* F.GetAt(1, 1) - F.GetAt(1, 0) * F.GetAt(0, 1);

	Chaos::PMatrix<T, 3, 3> D = (S.GetAt(0, 0) + S.GetAt(1, 1) + S.GetAt(2, 2)) * Chaos::PMatrix<T, 3, 3>::Identity - S;
	Chaos::PMatrix<T, 3, 3> DInv = D.Inverse();

	for (int32 i = 0; i < 3; i++) 
	{
		for (int32 j = 0; j < 3; j++)
		{
			DInvCache[i * 3 + j] = DInv.GetAt(i, j);
			RCache[i * 3 + j] = R.GetAt(i, j);
		}
	}

	JCache = F.Determinant();

}

template<typename T>
void CorotatedCache<T>::P(const Chaos::PMatrix<T, 3, 3>& F, const T mu, const T lambda, Chaos::PMatrix<T, 3, 3>& P) 
{
	Chaos::PMatrix<T, 3, 3> R((T)0.), JFinvT((T)0.);
	for (int32 i = 0; i < 3; i++)
	{
		for (int32 j = 0; j < 3; j++)
		{
			R.SetAt(i, j, RCache[i * 3 + j]);
			JFinvT.SetAt(i, j, JFInvTCache[i * 3 + j]);
		}
	}
	P = (T)2. * mu * (F - R) + lambda * (JCache - (T)1.) * JFinvT;
}

template<typename T>
void CorotatedCache<T>::deltaP(const Chaos::PMatrix<T, 3, 3>& F, const Chaos::PMatrix<T, 3, 3>& dF, const T mu, const T lambda, Chaos::PMatrix<T, 3, 3>& dP) 
{
	Chaos::PMatrix<T, 3, 3> R((T)0.), S((T)0.), JFinvT((T)0.), dR((T)0.), dJFinvT((T)0.), Dinv((T)0.);
	for (int32 i = 0; i < 3; i++) 
	{
		for (int32 j = 0; j < 3; j++)
		{
			R.SetAt(i, j, RCache[i * 3 + j]);
			JFinvT.SetAt(i, j, JFInvTCache[i * 3 + j]);
			Dinv.SetAt(i, j, DInvCache[i * 3 + j]);
		}
	}

	T dJ = (T)0.;
	for (int32 alpha = 0; alpha < 3; alpha++) {
		for (int32 beta = 0; beta < 3; beta++) {
			dJ += JFinvT.GetAt(alpha, beta) * dF.GetAt(alpha, beta);
		}
	}

	//dR
	Chaos::PMatrix<T, 3, 3> A = dF * R.GetTransposed(), B((T)0.);
	TVector<T, 3> a((T)0.), b((T)0.);
	b[0] = A.GetAt(1, 2) - A.GetAt(2, 1); b[1] = A.GetAt(2, 0) - A.GetAt(0, 2); b[2] = A.GetAt(0, 1) - A.GetAt(1, 0);
	a = Dinv.GetTransposed() * b;
	B.SetAt(0, 0, (T)0.); B.SetAt(0, 1, a[2]); B.SetAt(0, 2, -a[1]);
	B.SetAt(1, 0, -a[2]); B.SetAt(1, 1, (T)0.); B.SetAt(1, 2, a[0]);
	B.SetAt(2, 0, a[1]); B.SetAt(2, 1, -a[0]); B.SetAt(2, 2, (T)0.);
	dR = B * R;

	dJFinvT.SetAt(0, 0, F.GetAt(2, 2) * dF.GetAt(1, 1) - F.GetAt(2, 1) * dF.GetAt(1, 2) - F.GetAt(1, 2) * dF.GetAt(2, 1) + F.GetAt(1, 1) * dF.GetAt(2, 2));
	dJFinvT.SetAt(0, 1, -F.GetAt(2, 2) * dF.GetAt(1, 0) + F.GetAt(2, 0) * dF.GetAt(1, 2) + F.GetAt(1, 2) * dF.GetAt(2, 0) - F.GetAt(1, 0) * dF.GetAt(2, 2));
	dJFinvT.SetAt(0, 2, F.GetAt(2, 1) * dF.GetAt(1, 0) - F.GetAt(2, 0) * dF.GetAt(1, 1) - F.GetAt(1, 1) * dF.GetAt(2, 0) + F.GetAt(1, 0) * dF.GetAt(2, 1));
	dJFinvT.SetAt(1, 0, -F.GetAt(2, 2) * dF.GetAt(0, 1) + F.GetAt(2, 1) * dF.GetAt(0, 2) + F.GetAt(0, 2) * dF.GetAt(2, 1) - F.GetAt(0, 1) * dF.GetAt(2, 2));
	dJFinvT.SetAt(1, 1, F.GetAt(2, 2) * dF.GetAt(0, 0) - F.GetAt(2, 0) * dF.GetAt(0, 2) - F.GetAt(0, 2) * dF.GetAt(2, 0) + F.GetAt(0, 0) * dF.GetAt(2, 2));
	dJFinvT.SetAt(1, 2, -F.GetAt(2, 1) * dF.GetAt(0, 0) + F.GetAt(2, 0) * dF.GetAt(0, 1) + F.GetAt(0, 1) * dF.GetAt(2, 0) - F.GetAt(0, 0) * dF.GetAt(2, 1));
	dJFinvT.SetAt(2, 0, F.GetAt(1, 2) * dF.GetAt(0, 1) - F.GetAt(1, 1) * dF.GetAt(0, 2) - F.GetAt(0, 2) * dF.GetAt(1, 1) + F.GetAt(0, 1) * dF.GetAt(1, 2));
	dJFinvT.SetAt(2, 1, -F.GetAt(1, 2) * dF.GetAt(0, 0) + F.GetAt(1, 0) * dF.GetAt(0, 2) + F.GetAt(0, 2) * dF.GetAt(1, 0) - F.GetAt(0, 0) * dF.GetAt(1, 2));
	dJFinvT.SetAt(2, 2, F.GetAt(1, 1) * dF.GetAt(0, 0) - F.GetAt(1, 0) * dF.GetAt(0, 1) - F.GetAt(0, 1) * dF.GetAt(1, 0) + F.GetAt(0, 0) * dF.GetAt(1, 1));

	dP = (T)2. * mu * (dF - dR) + lambda * dJ * JFinvT + lambda * (JCache - (T)1.) * dJFinvT;

}

template <typename T>
T PsiCorotated(const Chaos::PMatrix<T, 3, 3>& F, const T mu, const T lambda) 
{
	Chaos::PMatrix<T, 3, 3> R((T)0.), S((T)0.), b((T)0.);
	Chaos::PolarDecomposition(F, R, S);
	T J = F.Determinant();
	b = F * F.GetTransposed();
	T STrace = T(0), bTrace = T(0);
	for (int32 alpha = 0; alpha < 3; alpha++)
	{
		STrace += S.GetDiagonal()[alpha];
		bTrace += b.GetDiagonal()[alpha];
	}
	
	return mu * (bTrace - T(2) * STrace + T(3)) + lambda * (J - T(1)) * (J - T(1)) / T(2);
}

template <typename T>
void PCorotated(const Chaos::PMatrix<T, 3, 3>& Fe, const T mu, const T lambda, Chaos::PMatrix<T, 3, 3>& P)
{
	Chaos::PMatrix<T, 3, 3> R((T)0.), S((T)0.), JFinvT((T)0.);
	Chaos::PolarDecomposition(Fe, R, S);

	JFinvT.SetAt(0, 0, Fe.GetAt(1, 1) * Fe.GetAt(2, 2) - Fe.GetAt(2, 1) * Fe.GetAt(1, 2));
	JFinvT.SetAt(0, 1, Fe.GetAt(2, 0) * Fe.GetAt(1, 2) - Fe.GetAt(1, 0) * Fe.GetAt(2, 2));
	JFinvT.SetAt(0, 2, Fe.GetAt(1, 0) * Fe.GetAt(2, 1) - Fe.GetAt(2, 0) * Fe.GetAt(1, 1));
	JFinvT.SetAt(1, 0, Fe.GetAt(2, 1) * Fe.GetAt(0, 2) - Fe.GetAt(0, 1) * Fe.GetAt(2, 2));
	JFinvT.SetAt(1, 1, Fe.GetAt(0, 0) * Fe.GetAt(2, 2) - Fe.GetAt(2, 0) * Fe.GetAt(0, 2));
	JFinvT.SetAt(1, 2, Fe.GetAt(2, 0) * Fe.GetAt(0, 1) - Fe.GetAt(0, 0) * Fe.GetAt(2, 1));
	JFinvT.SetAt(2, 0, Fe.GetAt(0, 1) * Fe.GetAt(1, 2) - Fe.GetAt(1, 1) * Fe.GetAt(0, 2));
	JFinvT.SetAt(2, 1, Fe.GetAt(1, 0) * Fe.GetAt(0, 2) - Fe.GetAt(0, 0) * Fe.GetAt(1, 2));
	JFinvT.SetAt(2, 2, Fe.GetAt(0, 0) * Fe.GetAt(1, 1) - Fe.GetAt(1, 0) * Fe.GetAt(0, 1));

	T J = Fe.Determinant();

	P = (T)2. * mu * (Fe - R) + lambda * (J - (T)1.) * JFinvT;
}

} 

template CHAOS_API void Chaos::Softs::CorotatedCache<Chaos::FRealSingle>::UpdateCache(const Chaos::PMatrix<Chaos::FRealSingle, 3, 3>&);
template CHAOS_API void Chaos::Softs::CorotatedCache<Chaos::FRealSingle>::P(const Chaos::PMatrix<Chaos::FRealSingle, 3, 3>&, const Chaos::FRealSingle, const Chaos::FRealSingle, Chaos::PMatrix<Chaos::FRealSingle, 3, 3>&);
template CHAOS_API void Chaos::Softs::CorotatedCache<Chaos::FRealSingle>::deltaP(const Chaos::PMatrix<Chaos::FRealSingle, 3, 3>&, const Chaos::PMatrix<Chaos::FRealSingle, 3, 3>&, const Chaos::FRealSingle, const Chaos::FRealSingle, Chaos::PMatrix<Chaos::FRealSingle, 3, 3>&);
template CHAOS_API void Chaos::Softs::CorotatedCache<Chaos::FReal>::UpdateCache(const Chaos::PMatrix<Chaos::FReal, 3, 3>&);
template CHAOS_API void Chaos::Softs::CorotatedCache<Chaos::FReal>::P(const Chaos::PMatrix<Chaos::FReal, 3, 3>&, const Chaos::FReal, const Chaos::FReal, Chaos::PMatrix<Chaos::FReal, 3, 3>&);
template CHAOS_API void Chaos::Softs::CorotatedCache<Chaos::FReal>::deltaP(const Chaos::PMatrix<Chaos::FReal, 3, 3>&, const Chaos::PMatrix<Chaos::FReal, 3, 3>&, const Chaos::FReal, const Chaos::FReal, Chaos::PMatrix<Chaos::FReal, 3, 3>&);
template CHAOS_API void Chaos::Softs::PCorotated<Chaos::FReal>(const Chaos::PMatrix<Chaos::FReal, 3, 3>& Fe, const Chaos::FReal mu, const Chaos::FReal lambda, Chaos::PMatrix<Chaos::FReal, 3, 3>& P);
template CHAOS_API void Chaos::Softs::PCorotated<Chaos::FRealSingle>(const Chaos::PMatrix<Chaos::FRealSingle, 3, 3>& Fe, const Chaos::FRealSingle mu, const Chaos::FRealSingle lambda, Chaos::PMatrix<Chaos::FRealSingle, 3, 3>& P);
template CHAOS_API Chaos::FReal Chaos::Softs::PsiCorotated<Chaos::FReal>(const Chaos::PMatrix<Chaos::FReal, 3, 3>& F, const Chaos::FReal mu, const Chaos::FReal lambda);
template CHAOS_API Chaos::FRealSingle Chaos::Softs::PsiCorotated<Chaos::FRealSingle>(const Chaos::PMatrix<Chaos::FRealSingle, 3, 3>& F, const Chaos::FRealSingle mu, const Chaos::FRealSingle lambda);
