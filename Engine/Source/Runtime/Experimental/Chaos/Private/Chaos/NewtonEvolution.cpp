// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/NewtonEvolution.h"

#include "Chaos/Framework/Parallel.h"
#include "Chaos/PBDTriangleMeshIntersections.h"
#include "Chaos/PerParticleDampVelocity.h"
#include "Chaos/PerParticleGravity.h"
#include "Chaos/PerParticlePBDCCDCollisionConstraint.h"
#include "Chaos/PerParticlePBDCollisionConstraint.h"
#include "Chaos/VelocityField.h"
#include "ChaosStats.h"
#include "HAL/IConsoleManager.h"
#include "Chaos/NewtonCorotatedCache.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "HAL/PlatformFileManager.h"
#include "Chaos/Math/Krylov.h"

//DECLARE_CYCLE_STAT(TEXT("Chaos Newton Advance Time"), STAT_ChaosNewtonVAdvanceTime, STATGROUP_Chaos);
////DECLARE_CYCLE_STAT(TEXT("Chaos Newton Velocity Damping State Update"), STAT_ChaosNewtonVelocityDampUpdateState, STATGROUP_Chaos);
//DECLARE_CYCLE_STAT(TEXT("Chaos Newton Velocity Field Update Forces"), STAT_ChaosNewtonVelocityFieldUpdateForces, STATGROUP_Chaos);
//DECLARE_CYCLE_STAT(TEXT("Chaos Newton Velocity Damping"), STAT_ChaosNewtonVelocityDampUpdate, STATGROUP_Chaos);
//DECLARE_CYCLE_STAT(TEXT("Chaos Newton Pre Iteration Updates"), STAT_ChaosNewtonPreIterationUpdates, STATGROUP_Chaos);
//DECLARE_CYCLE_STAT(TEXT("Chaos Newton Integrate"), STAT_ChaosClothSolverIntegrate, STATGROUP_Chaos);
//DECLARE_CYCLE_STAT(TEXT("Chaos Newton Iteration Loop"), STAT_ChaosNewtonIterationLoop, STATGROUP_Chaos);
//DECLARE_CYCLE_STAT(TEXT("Chaos Newton Post Iteration Updates"), STAT_ChaosNewtonPostIterationUpdates, STATGROUP_Chaos);
//DECLARE_CYCLE_STAT(TEXT("Chaos Newton Constraint Rules"), STAT_ChaosNewtonConstraintRule, STATGROUP_Chaos);
//DECLARE_CYCLE_STAT(TEXT("Chaos Newton Post Collision Constraint Rules"), STAT_ChaosNewtonPostCollisionConstraintRule, STATGROUP_Chaos);
//DECLARE_CYCLE_STAT(TEXT("Chaos Newton Self Collision"), STAT_ChaosNewtonSelfCollisionRule, STATGROUP_Chaos);
//DECLARE_CYCLE_STAT(TEXT("Chaos Newton Collision Rule"), STAT_ChaosNewtonCollisionRule, STATGROUP_Chaos);
//DECLARE_CYCLE_STAT(TEXT("Chaos Newton Collider Friction"), STAT_ChaosNewtonCollisionRuleFriction, STATGROUP_Chaos);
//DECLARE_CYCLE_STAT(TEXT("Chaos Newton Collider Kinematic Update"), STAT_ChaosNewtonCollisionKinematicUpdate, STATGROUP_Chaos);
//DECLARE_CYCLE_STAT(TEXT("Chaos Newton Clear Collided Array"), STAT_ChaosNewtonClearCollidedArray, STATGROUP_Chaos);
//DECLARE_CYCLE_STAT(TEXT("Chaos Newton Constraints Init"), STAT_ChaosXNewtonConstraintsInit, STATGROUP_Chaos);

TAutoConsoleVariable<bool> CVarChaosNewtonEvolutionUseNestedParallelFor(TEXT("p.Chaos.NewtonEvolution.UseNestedParallelFor"), true, TEXT(""), ECVF_Cheat);
TAutoConsoleVariable<bool> CVarChaosNewtonEvolutionFastPositionBasedFriction(TEXT("p.Chaos.NewtonEvolution.FastPositionBasedFriction"), true, TEXT(""), ECVF_Cheat);
TAutoConsoleVariable<bool> CVarChaosNewtonEvolutionUseSmoothTimeStep(TEXT("p.Chaos.NewtonEvolution.UseSmoothTimeStep"), true, TEXT(""), ECVF_Cheat);
TAutoConsoleVariable<int32> CVarChaosNewtonEvolutionMinParallelBatchSize(TEXT("p.Chaos.NewtonEvolution.MinParallelBatchSize"), 300, TEXT(""), ECVF_Cheat);
TAutoConsoleVariable<bool> CVarChaosNewtonEvolutionWriteCCDContacts(TEXT("p.Chaos.NewtonEvolution.WriteCCDContacts"), false, TEXT("Write CCD collision contacts and normals potentially causing the CCD collision threads to lock, allowing for debugging of these contacts."), ECVF_Cheat);
TAutoConsoleVariable<bool> CVarChaosNewtonEvolutionParallelIntegrate(TEXT("p.Chaos.NewtonEvolution.ParalleIntegrate"), false, TEXT("Run the integration step in parallel for."), ECVF_Cheat);

//#if INTEL_ISPC && !UE_BUILD_SHIPPING
//bool bChaos_PostIterationUpdates_ISPC_Enabled = true;
//FAutoConsoleVariableRef CVarChaosPostIterationUpdatesISPCEnabled(TEXT("p.Chaos.PostIterationUpdates.ISPC"), bChaos_PostIterationUpdates_ISPC_Enabled, TEXT("Whether to use ISPC optimizations in Newton Post iteration updates"));
//
//static_assert(sizeof(ispc::FVector3f) == sizeof(Chaos::Softs::FSolverVec3), "sizeof(ispc::FVector3f) != sizeof(Chaos::Softs::FSolverVec3");
//#endif

namespace Chaos::Softs {

void FNewtonEvolution::AddGroups(int32 NumGroups)
{
	// Add elements
	const uint32 Offset = TArrayCollection::Size();
	TArrayCollection::AddElementsHelper(NumGroups);

	// Set defaults
	for (uint32 GroupId = Offset; GroupId < TArrayCollection::Size(); ++GroupId)
	{
		MGroupGravityAccelerations[GroupId] = MGravity;
		MGroupCollisionThicknesses[GroupId] = MCollisionThickness;
		MGroupCoefficientOfFrictions[GroupId] = MCoefficientOfFriction;
		MGroupDampings[GroupId] = MDamping;
		MGroupLocalDampings[GroupId] = MLocalDamping;
		MGroupUseCCDs[GroupId] = false;
	}
}

void FNewtonEvolution::ResetGroups()
{
	TArrayCollection::ResizeHelper(0);
	AddGroups(1);  // Add default group
}

FNewtonEvolution::FNewtonEvolution(
	FSolverParticles&& InParticles,
	FSolverCollisionParticles&& InGeometryParticles,
	TArray<TVec3<int32>>&& CollisionTriangles,
	const TArray<TVector<int32, 4>>& InMesh,
	TArray<TArray<TVector<int32, 2>>>&& InIncidentElements,
	int32 NumNewtonIterations,
	int32 NumCGIterations,
	const TArray<int32>& ConstrainedVertices,
	const TArray<FSolverVec3>& BCPositions,
	FSolverReal CollisionThickness,
	FSolverReal SelfCollisionsThickness,
	FSolverReal CoefficientOfFriction,
	FSolverReal Damping,
	FSolverReal LocalDamping,
	FSolverReal EMesh,
	FSolverReal NuMesh,
	FSolverReal NewtonTol,
	FSolverReal CGTolIn,
	bool bWriteDebugInfoIn)
	: MParticles(MoveTemp(InParticles))
	, MParticlesActiveView(MParticles)
	, MCollisionParticles(MoveTemp(InGeometryParticles))
	, MCollisionParticlesActiveView(MCollisionParticles)
	, MConstraintInitsActiveView(MConstraintInits)
	, MConstraintRulesActiveView(MConstraintRules)
	, MPostCollisionConstraintRulesActiveView(MPostCollisionConstraintRules)
	, MNumNewtonIterations(NumNewtonIterations)
	, MNumCGIterations(NumCGIterations)
	, MMesh(InMesh)
	, MIncidentElements(MoveTemp(InIncidentElements))
	, MGravity(FSolverVec3((FSolverReal)0., (FSolverReal)0., (FSolverReal)980.665))
	, MCollisionThickness(CollisionThickness)
	, MCoefficientOfFriction(CoefficientOfFriction)
	, MDamping(Damping)
	, MLocalDamping(LocalDamping)
	, MTime(0)
	, MSmoothDt(1.f / 30.f)  // Initialize filtered timestep at 30fps
	, MNewtonTol(NewtonTol)
	, MCGTol(CGTolIn)
	, MConstrainedVertices(ConstrainedVertices)
	, MBCPositions(BCPositions)
	, bWriteDebugInfo(bWriteDebugInfoIn)
{
	// Add group arrays
	TArrayCollection::AddArray(&MGroupGravityAccelerations);
	TArrayCollection::AddArray(&MGroupVelocityAndPressureFields);
	TArrayCollection::AddArray(&MGroupForceRules);
	TArrayCollection::AddArray(&MGroupCollisionThicknesses);
	TArrayCollection::AddArray(&MGroupCoefficientOfFrictions);
	TArrayCollection::AddArray(&MGroupDampings);
	TArrayCollection::AddArray(&MGroupLocalDampings);
	TArrayCollection::AddArray(&MGroupUseCCDs);
	AddGroups(1);  // Add default group

	// Add particle arrays
	MParticles.AddArray(&MParticleGroupIds);
	MCollisionParticles.AddArray(&MCollisionTransforms);
	MCollisionParticles.AddArray(&MCollided);
	MCollisionParticles.AddArray(&MCollisionParticleGroupIds);
	ConsCaches.CorotatedCache.Init(CorotatedCache<FSolverReal>(), InMesh.Num());

	Lambda = EMesh * NuMesh / (((FSolverReal)1. + NuMesh) * ((FSolverReal)1. - (FSolverReal)2. * NuMesh));
	Mu = EMesh / ((FSolverReal)2. * ((FSolverReal)1. + NuMesh));
	
	MVn.Init(FSolverVec3(0.f), MParticles.Size());

	UpdatePositionBasedState = [this](const bool update_damping = false) {
		PhysicsParallelFor(MMesh.Num(), [&](const int32 e) {
			Chaos::PMatrix<FSolverReal, 3, 3> Fe = MEFem->F(e, MParticles);
			ConsCaches.CorotatedCache[e].UpdateCache(Fe);
		});
	};

	PStress = [this](const PMatrix<FSolverReal, 3, 3>& F, PMatrix<FSolverReal, 3, 3>& P, const int32 e) {
		ConsCaches.CorotatedCache[e].P(F, Mu, Lambda, P);
	};

	DeltaPStress = [this](const PMatrix<FSolverReal, 3, 3>& F, const PMatrix<FSolverReal, 3, 3>& deltaF, PMatrix<FSolverReal, 3, 3>& deltaP, const int32 e) {
		ConsCaches.CorotatedCache[e].deltaP(F, deltaF, Mu, Lambda, deltaP);
	};
	

	//set up lambdas from constraint info
	ProjectBCs = [this](TArray<FSolverVec3>& y) {
		for (int32 j = 0; j < this->MConstrainedVertices.Num(); j++) {
			y[this->MConstrainedVertices[j]] = FSolverVec3(0.f);
		}
	};

	SetBCs = [this](const FSolverReal time, TArray<FSolverVec3>& y) {
		for (int32 j = 0; j < this->MConstrainedVertices.Num(); j++) {
			y[this->MConstrainedVertices[j]] = this->MBCPositions[j];
		}
	};

	MAddExternalForce = [this](const FSolverReal time, const FSolverReal dt, const TArray<FSolverReal>& nodal_mass, bool divide_mass, TArray<FSolverVec3>& v) {
		//TODO: make following parallel
		for (int32 i = 0; i < v.Num(); ++i) {
			for (int32 j = 0; j < 3; j++) {
				if (divide_mass) {
					v[i][j] -= dt * MGravity[j];
				}
				else {
					v[i][j] -= dt * MGravity[j] * nodal_mass[i];
				}
  		}
		}
	};

	FString file = FPaths::ProjectDir();
	file.Append(TEXT("/HoudiniOutput/NewtonLog.txt"));
	FFileHelper::SaveStringToFile(FString("Running Newton's Method\r\n"), *file);
	FFileHelper::SaveStringToFile(FString("Current Configuration:\r\n"), *file, FFileHelper::EEncodingOptions::AutoDetect, &IFileManager::Get(), EFileWrite::FILEWRITE_Append);
	FFileHelper::SaveStringToFile((FString("Newton Tol = ") + FString::SanitizeFloat(MNewtonTol) + FString("\r\n")), *file, FFileHelper::EEncodingOptions::AutoDetect, &IFileManager::Get(), EFileWrite::FILEWRITE_Append);
	FFileHelper::SaveStringToFile((FString("CG Tol = ") + FString::SanitizeFloat(MCGTol) + FString("\r\n")), *file, FFileHelper::EEncodingOptions::AutoDetect, &IFileManager::Get(), EFileWrite::FILEWRITE_Append);
	FFileHelper::SaveStringToFile((FString("Max Newton Iters = ") + FString::FromInt(MNumNewtonIterations) + FString("\r\n")), *file, FFileHelper::EEncodingOptions::AutoDetect, &IFileManager::Get(), EFileWrite::FILEWRITE_Append);
	FFileHelper::SaveStringToFile((FString("Max CG Iters = ") + FString::FromInt(MNumCGIterations) + FString("\r\n")), *file, FFileHelper::EEncodingOptions::AutoDetect, &IFileManager::Get(), EFileWrite::FILEWRITE_Append);
	FFileHelper::SaveStringToFile(FString("ConsModel: Corotated\r\n"), *file, FFileHelper::EEncodingOptions::AutoDetect, &IFileManager::Get(), EFileWrite::FILEWRITE_Append);

}

void FNewtonEvolution::InitFEM() {
	MEFem.Reset(new ElasticFEM<FSolverReal, FSolverParticles>(MParticles, MMesh, MIncidentElements));
	Measure = MEFem->Measure;
}


void FNewtonEvolution::WriteOutputLog(const int32 Frame) 
{
	FString file = FPaths::ProjectDir();
	file.Append(TEXT("/HoudiniOutput/NewtonLog.txt"));
	FFileHelper::SaveStringToFile(FString("---------------------------------------------------\r\n"), *file, FFileHelper::EEncodingOptions::AutoDetect, &IFileManager::Get(), EFileWrite::FILEWRITE_Append);
	FFileHelper::SaveStringToFile((FString("Frame = ") + FString::FromInt(Frame) + FString("\r\n")), *file, FFileHelper::EEncodingOptions::AutoDetect, &IFileManager::Get(), EFileWrite::FILEWRITE_Append);
	FFileHelper::SaveStringToFile((FString("Time = ") + FString::SanitizeFloat(MTime) + FString("\r\n")), *file, FFileHelper::EEncodingOptions::AutoDetect, &IFileManager::Get(), EFileWrite::FILEWRITE_Append);
	FFileHelper::SaveStringToFile(FString("---------------------------------------------------\r\n"), *file, FFileHelper::EEncodingOptions::AutoDetect, &IFileManager::Get(), EFileWrite::FILEWRITE_Append);
}

	void FNewtonEvolution::ResetParticles()
	{
		// Reset particles
		MParticles.Resize(0);
		MParticlesActiveView.Reset();

		// Reset particle groups
		ResetGroups();
	}

	int32 FNewtonEvolution::AddParticleRange(int32 NumParticles, uint32 GroupId, bool bActivate)
	{
		if (NumParticles)
		{
			const int32 Offset = (int32)MParticles.Size();

			MParticles.AddParticles(NumParticles);

			MVn.Init(FSolverVec3(0.f), MParticles.Size());

			// Initialize the new particles' group ids
			for (int32 i = Offset; i < (int32)MParticles.Size(); ++i)
			{
				MParticleGroupIds[i] = GroupId;
			}

			// Resize the group parameter arrays
			const uint32 GroupSize = TArrayCollection::Size();
			if (GroupId >= GroupSize)
			{
				AddGroups(GroupId + 1 - GroupSize);
			}

			// Add range
			MParticlesActiveView.AddRange(NumParticles, bActivate);

			return Offset;
		}
		return INDEX_NONE;
	}

	void FNewtonEvolution::ResetCollisionParticles(int32 NumParticles)
	{
		MCollisionParticles.Resize(NumParticles);
		MCollisionParticlesActiveView.Reset(NumParticles);
	}

	int32 FNewtonEvolution::AddCollisionParticleRange(int32 NumParticles, uint32 GroupId, bool bActivate)
	{
		if (NumParticles)
		{
			const int32 RangeOffset = (int32)MCollisionParticles.Size();

			MCollisionParticles.AddParticles(NumParticles);

			// Initialize the new particles' group ids
			for (int32 i = RangeOffset; i < (int32)MCollisionParticles.Size(); ++i)
			{
				MCollisionParticleGroupIds[i] = GroupId;
			}

			// Add range
			MCollisionParticlesActiveView.AddRange(NumParticles, bActivate);

			return RangeOffset;
		}
		return INDEX_NONE;
	}

	int32 FNewtonEvolution::AddConstraintInitRange(int32 NumConstraints, bool bActivate)
	{
		// Add new constraint init functions
		MConstraintInits.AddDefaulted(NumConstraints);

		// Add range
		return MConstraintInitsActiveView.AddRange(NumConstraints, bActivate);
	}

	int32 FNewtonEvolution::AddConstraintRuleRange(int32 NumConstraints, bool bActivate)
	{
		// Add new constraint rule functions
		MConstraintRules.AddDefaulted(NumConstraints);

		// Add range
		return MConstraintRulesActiveView.AddRange(NumConstraints, bActivate);
	}

	int32 FNewtonEvolution::AddPostCollisionConstraintRuleRange(int32 NumConstraints, bool bActivate)
	{
		// Add new constraint rule functions
		MPostCollisionConstraintRules.AddDefaulted(NumConstraints);

		// Add range
		return MPostCollisionConstraintRulesActiveView.AddRange(NumConstraints, bActivate);
	}

	template <typename Func1, typename Func2>
	void FNewtonEvolution::ComputeNegativeBackwardEulerResidual(const FSolverParticles& InParticles, const TArray<TArray<TVector<int32, 2>>>& IncidentElements, const TArray<FSolverReal>& NodalMass, const TArray<FSolverVec3>& Vn, const FSolverParticles& Xn, Func1 P, Func2 AddExternalForce, const FSolverReal Time, const FSolverReal Dt, TArray<FSolverVec3>& Residual)
	{
		//std::fill(std::execution::par_unseq, residual.begin(), residual.end(), Vector());
		MEFem->AddInternalElasticForce(InParticles, P, Residual, &IncidentElements);
		//wc.AddInternalForce(x, residual);
		AddExternalForce(Time + Dt, (FSolverReal)1., NodalMass, false, Residual);

		const FSolverReal DtSquared = Dt * Dt;
//#pragma omp parallel for
		for (int32 i = 0; i < int32(InParticles.Size()); ++i) {
			for (int32 alpha = 0; alpha < 3; alpha++) {
				Residual[i][alpha] *= DtSquared;
				Residual[i][alpha] -= NodalMass[i] * (InParticles.GetP(i)[alpha] - InParticles.GetX(i)[alpha] - Dt * Vn[i][alpha]);
			}
		}
	}

	FSolverReal ComputeVectorL2Norm(const TArray<FSolverVec3>& InVector)
	{
		FSolverReal Sum = (FSolverReal)0.;
		for (FSolverVec3 Vector : InVector)
		{
			for (int32 i = 0; i < 3; i++)
				Sum += Vector[i] * Vector[i];
		}
		return FMath::Sqrt(Sum);
	}



	template<typename Func1, typename Func2, typename Func3, typename Func4, typename Func5, typename Func6>
	void FNewtonEvolution::DoNewtonStep(const int32 max_it_newton, const FSolverReal newton_tol, const int32 max_it_cg, const FSolverReal cg_tol, Func1 P, Func2 dP, const FSolverReal Time, const FSolverReal Dt, const TArray<TArray<TVector<int32, 2>>>& IncidentElements, const TArray<FSolverReal>& nodal_mass, Func3 set_bcs, Func4 project_bcs, Func5 add_external_force, Func6 update_position_based_state, FSolverParticles& InParticles, TArray<FSolverReal>& ResidualNorm, bool use_cg, FSolverReal cg_prctg_reduce, bool no_verbose) {
		FString file = FPaths::ProjectDir();
		file.Append(TEXT("/HoudiniOutput/NewtonLog.txt"));
		
		for (int32 it = 0; it < MNumNewtonIterations; it++)
		{
			/////////////////////////
			//compute Newton Residual
			/////////////////////////
			TArray<FSolverVec3> Residual;
			Residual.Init(FSolverVec3(0.f), InParticles.Size());
			update_position_based_state(false);

			ComputeNegativeBackwardEulerResidual(InParticles, IncidentElements, nodal_mass,  MVn, InParticles, P, add_external_force, Time, Dt, Residual);
			project_bcs(Residual);

			//when possible, discard unused nodes in loops
			FSolverReal Norm = ComputeVectorL2Norm(Residual);

			FFileHelper::SaveStringToFile((FString("Newton Residual two Norm = ") + FString::SanitizeFloat(Norm) + FString("\r\n")), *file, FFileHelper::EEncodingOptions::AutoDetect, &IFileManager::Get(), EFileWrite::FILEWRITE_Append);
			if (Norm < newton_tol) {
				FFileHelper::SaveStringToFile((FString("Newton converged to tolerance ") + FString::SanitizeFloat(newton_tol) + FString(" in = ") + FString::FromInt(it) + FString("\r\n")), *file, FFileHelper::EEncodingOptions::AutoDetect, &IFileManager::Get(), EFileWrite::FILEWRITE_Append);
				//std::cout << "Newton converged to tolerance " << newton_tol << " in = " << it << "iterations" << std::endl;
				return;
			}


			auto multiply = [this, &project_bcs, &dP, &nodal_mass, &Dt](TArray<FSolverVec3>& y, const TArray<FSolverVec3>& _x) {
				//TODO: this is not strictly necessary since P*_x should be _x in the context of CG
				TArray<FSolverVec3> x_proj = _x;
				project_bcs(x_proj);

				y.Init(FSolverVec3((FSolverReal)0.), x_proj.Num());

				//defining the damping delta stress lambda to be aware of Dt:
				this->MEFem->AddNegativeInternalElasticForceDifferential(this->MParticles, dP, x_proj, y, &(this->MIncidentElements));

				//wc.AddNegativeInternalForceDifferential(x_proj, y);


				//add effect of mass matrix and scale by Dt^2
				if (!use_list) {
					PhysicsParallelFor(y.Num(), [&](const int32 i) {
						for (int32 c = 0; c < 3; c++) {
							y[i][c] = nodal_mass[i] * x_proj[i][c] + Dt * Dt * y[i][c];
						}
						});
				}
				else {
					//use_list will specify a sparser set of nodes for the loop
					PhysicsParallelFor(use_list->Num(), [&](const int32 i) {
						int32 j = (*use_list)[i];
						for (int32 c = 0; c < 3; c++) {
							y[j][c] = nodal_mass[j] * x_proj[j][c] + Dt * Dt * y[j][c];
						}
						});
				}

				project_bcs(y);
			};

			/////////////////////
			//define PCG lambdas
			/////////////////////
			auto multiply_F = [this, &nodal_mass, &Dt](TArray<FSolverVec3>& Y, const TArray<FSolverVec3>& X) {
				// TGSLAssert(Y.size() == X.size() && X.size() == nodal_mass.size(), "BackwardEuler: attempting multiply precoditioners between vectors of differing size\n");

				PhysicsParallelFor(X.Num(), [&](const int32 i) {
					if (nodal_mass[i] > (FSolverReal)0.) {
						for (int32 j = 0; j < 3; j++) {
							Y[i][j] = Dt * X[i][j] / (FMath::Sqrt(nodal_mass[i]));
						}
					}
					else {
						for (int32 j = 0; j < 3; j++) {
							Y[i][j] = Dt * X[i][j];
						}
					}
					});
			};

			auto AddScaled = [&](TArray<FSolverVec3>& a, const TArray<FSolverVec3>& b, const FSolverReal s) {
				//TGSLAssert(a.size() == b.size(), "attempting addScaled with vectors of differing size\n");
				if (!use_list) {
					PhysicsParallelFor(a.Num(), [&](const int32 i) {
						for (int32 j = 0; j < 3; j++)
							a[i][j] += b[i][j] * s;
						});
				}
				else {

					PhysicsParallelFor(use_list->Num(), [&](const int32 i) {
						int32 j = (*use_list)[i];
						for (size_t alpha = 0; alpha < 3; alpha++)
							a[j][alpha] += b[j][alpha] * s;
						});
				}
			};

			auto DotProduct = [&](const TArray<FSolverVec3>& X, const TArray<FSolverVec3>& Y) {
				//TGSLAssert(Y.size() == X.size(), "attempting dot product with vectors of differing size\n");
				FSolverReal sum = 0.f;
				//TV sum_omp(omp_get_max_threads(), 0);
				//TODO(Yizhou C): Make following parallel if possible
				if (!use_list) {
					for (int32 i = 0; i < X.Num(); i++) {
						for (int32 j = 0; j < 3; j++) {
							sum += X[i][j] * Y[i][j];
						}
					}
				}
				else
				{
					for (int32 i = 0; i < use_list->Num(); i++) {
						int32 j = (*use_list)[i];
						for (int32 alpha = 0; alpha < 3; alpha++) {
							sum += X[j][alpha] * Y[j][alpha];
						}
					}
				}

				return sum;
			};

			auto ComputeLinearResidual = [&](FSolverReal& r, const TArray<FSolverVec3>& x, const TArray<FSolverVec3>& b) {
				TArray<FSolverVec3> res, Fres;
				res.Init(FSolverVec3((FSolverReal)0.), x.Num());
				Fres.Init(FSolverVec3((FSolverReal)0.), x.Num());
				multiply(res, x);
				AddScaled(res, b, FSolverReal(-1));
				multiply_F(Fres, res);
				r = FSolverReal(0.); r = FMath::Sqrt(DotProduct(Fres, Fres));
			};

			auto Set = [&](TArray<FSolverVec3>& X, const TArray<FSolverVec3>& Y) {
				//TGSLAssert(Y.size() == X.size(), "attempting set with vectors of differing size\n");
				if (!use_list) {
					PhysicsParallelFor(X.Num(), [&](const int32 i) {
						for (int32 j = 0; j < 3; j++)
							X[i][j] = Y[i][j];
						});
				}
				else {
					PhysicsParallelFor(use_list->Num(), [&](const int32 i) {
						int32 j = (*use_list)[i];
						for (int32 alpha = 0; alpha < 3; alpha++)
							X[j][alpha] = Y[j][alpha];
						});
				}
			};

			auto SetScaled = [&](TArray<FSolverVec3>& a, const TArray<FSolverVec3>& b, FSolverReal s) {
				//TGSLAssert(a.size() == b.size(), "attempting setScaled with vectors of differing size\n");
				if (!use_list) {
					PhysicsParallelFor(a.Num(), [&](const int32 i) {
						for (int32 j = 0; j < 3; j++)
							a[i][j] = b[i][j] * s;
						});
				}
				else {

					PhysicsParallelFor(use_list->Num(), [&](const int32 i) {
						int32 j = (*use_list)[i];
						for (int32 alpha = 0; alpha < 3; alpha++)
							a[j][alpha] = b[j][alpha] * s;
						});
				}
			};


			/////////////////////////
			//solve linearized system
			/////////////////////////
			TArray<FSolverVec3> delta_x;
			delta_x.Init(FSolverVec3(0.f), InParticles.Size());

			//Chaos::LanczosCG<FSolverReal>(multiply, delta_x, Residual, max_it_cg, cg_tol, use_list); 
			Chaos::LanczosCG(multiply, delta_x, Residual, max_it_cg, cg_tol, nullptr);
			////////////////////////
			//add Newton increment
			////////////////////////
			if (!use_list) {
				PhysicsParallelFor(InParticles.Size(), [&](const int32 i) {
					for (size_t alpha = 0; alpha < 3; alpha++) {
						InParticles.P(i)[alpha] += delta_x[i][alpha];
					}
					});
			}
			else {
				PhysicsParallelFor(use_list->Num(), [&](const int32 i) {
					int32 j = (*use_list)[i];
					for (size_t alpha = 0; alpha < 3; alpha++) {
						InParticles.P(j)[alpha] += delta_x[j][alpha];
					}
					});
			}

			////////////////////////////////
			//check linear Residual after cg
			////////////////////////////////
			TArray<FSolverVec3> LinearResidual;
			LinearResidual.Init(FSolverVec3((FSolverReal)0.), InParticles.Size());
			//LinearResidual.Init(InParticles.Size(), FSolverVec3(0.f));
			multiply(LinearResidual, delta_x);

			if (!use_list) {
				PhysicsParallelFor(InParticles.Size(), [&](const int32 i) {
					for (int32 alpha = 0; alpha < 3; alpha++) {
						LinearResidual[i][alpha] -= Residual[i][alpha];
					}
					});
			}
			else {
				PhysicsParallelFor(use_list->Num(), [&](const int32 i) {
					int32 j = (*use_list)[i];
					for (int32 alpha = 0; alpha < 3; alpha++) {
						LinearResidual[j][alpha] -= Residual[j][alpha];
					}
					});
			}

			Norm = ComputeVectorL2Norm(LinearResidual);
			FFileHelper::SaveStringToFile((FString("Linear Residual two Norm = ") + FString::SanitizeFloat(Norm) + FString("\r\n")), *file, FFileHelper::EEncodingOptions::AutoDetect, &IFileManager::Get(), EFileWrite::FILEWRITE_Append);
			//std::cout << "Linear Residual two Norm = " << std::scientific << Norm << std::endl;

			//if (!no_verbose)
			//	ResidualNorm[it] = Norm;

			//if (it == max_it_newton - 1 && hard_newton_tol)
			//	TGSLAssert(false, "ImplicitElasticFEM::BackwardEuler: Newton did not converge.");
		}

	}

	template<bool bForceRule, bool bVelocityField, bool bDampVelocityRule>
	void FNewtonEvolution::PreIterationUpdate(
		const FSolverReal Dt,
		const int32 Offset,
		const int32 Range,
		const int32 MinParallelBatchSize)
	{
		const uint32 ParticleGroupId = MParticleGroupIds[Offset];
		const TFunction<void(FSolverParticles&, const FSolverReal, const int32)>& ForceRule = MGroupForceRules[ParticleGroupId];
		const FSolverVec3& Gravity = MGroupGravityAccelerations[ParticleGroupId];
		FVelocityAndPressureField& VelocityAndPressureField = MGroupVelocityAndPressureFields[ParticleGroupId];

		if (bVelocityField)
		{
			//TRACE_CPUPROFILER_EVENT_SCOPE(ChaosNewtonVelocityFieldUpdateForces);
			//SCOPE_CYCLE_COUNTER(STAT_ChaosNewtonVelocityFieldUpdateForces);
			VelocityAndPressureField.UpdateForces(MParticles, Dt);  // Update force per surface element
		}

		FPerParticleDampVelocity DampVelocityRule(MGroupLocalDampings[ParticleGroupId]);
		if (bDampVelocityRule)
		{
			//TRACE_CPUPROFILER_EVENT_SCOPE(ChaosNewtonVelocityDampUpdateState);
			//SCOPE_CYCLE_COUNTER(STAT_ChaosNewtonVelocityDampUpdateState);
			DampVelocityRule.UpdatePositionBasedState(MParticles, Offset, Range);
		}


		{
			//TRACE_CPUPROFILER_EVENT_SCOPE(ChaosClothSolverIntegrate);
			//SCOPE_CYCLE_COUNTER(STAT_ChaosClothSolverIntegrate);

			constexpr FSolverReal DampingFrequency = (FSolverReal)60.;  // The damping value is the percentage of velocity removed per frame when running at 60Hz
			const FSolverReal Damping = FMath::Clamp(MGroupDampings[ParticleGroupId], (FSolverReal)0., (FSolverReal)1.);
			FSolverReal DampingPowDt;
			FSolverReal DampingIntegrated;
			if (Damping > (FSolverReal)1. - (FSolverReal)UE_KINDA_SMALL_NUMBER)
			{
				DampingIntegrated = DampingPowDt = (FSolverReal)0.;
			}
			else if (Damping > (FSolverReal)UE_SMALL_NUMBER)
			{
				const FSolverReal LogValueByFrequency = FMath::Loge((FSolverReal)1. - Damping) * DampingFrequency;

				DampingPowDt = FMath::Exp(LogValueByFrequency * Dt);  // DampingPowDt = FMath::Pow(OneMinusDamping, Dt * DampingFrequency);
				DampingIntegrated = (DampingPowDt - (FSolverReal)1.) / LogValueByFrequency;
			}
			else
			{
				DampingPowDt = (FSolverReal)1.;
				DampingIntegrated = Dt;
			}

			const int32 RangeSize = Range - Offset;
			PhysicsParallelFor(RangeSize,
				[this, &Offset, &ForceRule, &Gravity, &VelocityAndPressureField, &DampVelocityRule, DampingPowDt, DampingIntegrated, Dt](int32 i)
				{
					const int32 Index = Offset + i;
					if (MParticles.InvM(Index) != (FSolverReal)0.)  // Process dynamic particles
					{
						// Init forces with GravityForces
						MParticles.Acceleration(Index) = Gravity;

						// Force Rule
						if (bForceRule)
						{
							ForceRule(MParticles, Dt, Index); // F += M * A
						}

						// Velocity Field
						if (bVelocityField)
						{
							VelocityAndPressureField.Apply(MParticles, Dt, Index);
						}

						// Euler Step Velocity
						//original code:
						/*MParticles.V(Index) += MParticles.Acceleration(Index) * MSmoothDt;*/
						
						//new code:
						MParticles.V(Index) += MParticles.Acceleration(Index) * Dt;

						// Damp Velocity Rule
						if (bDampVelocityRule)
						{
							DampVelocityRule.ApplyFast(MParticles, Dt, Index);
						}
						
						//original code:
						// Euler Step with point damping integration
						//MParticles.P(Index) = MParticles.X(Index) + MParticles.V(Index) * DampingIntegrated;

						//MParticles.V(Index) *= DampingPowDt;

						//new code:
						MParticles.SetP(Index, MParticles.GetX(Index) + MParticles.GetV(Index) * Dt);

						MParticles.V(Index) *= 1.f;

					}
					else  // Process kinematic particles
					{
						MKinematicUpdate(MParticles, Dt, MTime, Index);
					}
				}, RangeSize < MinParallelBatchSize);
		}
	}

	void FNewtonEvolution::AdvanceOneTimeStep(const FSolverReal Dt, const bool bSmoothDt)
	{
		//TRACE_CPUPROFILER_EVENT_SCOPE(FNewtonEvolution_AdvanceOneTimeStep);
		//SCOPE_CYCLE_COUNTER(STAT_ChaosNewtonVAdvanceTime);

		// Advance time
		MTime += Dt;

		// Filter delta time to smoothen time variations and prevent unwanted vibrations, works best on Forces
		if (bSmoothDt && CVarChaosNewtonEvolutionUseSmoothTimeStep.GetValueOnAnyThread())
		{
			constexpr FSolverReal DeltaTimeDecay = (FSolverReal)0.1;
			MSmoothDt += (Dt - MSmoothDt) * DeltaTimeDecay;
		}
		else
		{
			MSmoothDt = Dt;
		}

		// Don't bother with threaded execution if we don't have enough work to make it worth while.
		const bool bUseSingleThreadedRange = !CVarChaosNewtonEvolutionUseNestedParallelFor.GetValueOnAnyThread();
		const int32 MinParallelBatchSize = !CVarChaosNewtonEvolutionParallelIntegrate.GetValueOnAnyThread() ?
			TNumericLimits<int32>::Max() :  // Disable
			CVarChaosNewtonEvolutionMinParallelBatchSize.GetValueOnAnyThread(); // TODO: 1000 is a guess, tune this!
		const bool bWriteCCDContacts = CVarChaosNewtonEvolutionWriteCCDContacts.GetValueOnAnyThread();

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ChaosNewtonPreIterationUpdates);

			MParticlesActiveView.RangeFor(
				[this, Dt, MinParallelBatchSize](FSolverParticles& Particles, int32 Offset, int32 Range)
				{
					const uint32 ParticleGroupId = MParticleGroupIds[Offset];

					if (MGroupVelocityAndPressureFields[ParticleGroupId].IsActive())
					{
						if (MGroupLocalDampings[ParticleGroupId] > (FSolverReal)0.)
						{
							if (MGroupForceRules[ParticleGroupId])  // VeloctiyFields, Damping, Forces  // Damping?????
							{
								PreIterationUpdate</*bForceRule =*/ true, /*bVelocityField =*/ true, /*bDampVelocityRule =*/ true>(Dt, Offset, Range, MinParallelBatchSize);
							}
							else  // VeloctiyFields, Damping
							{
								PreIterationUpdate</*bForceRule =*/ false, /*bVelocityField =*/ true, /*bDampVelocityRule =*/ true>(Dt, Offset, Range, MinParallelBatchSize);
							}
						}
						else  // No Damping
						{
							if (MGroupForceRules[ParticleGroupId])  // VeloctiyFields, Forces
							{
								PreIterationUpdate</*bForceRule =*/ true, /*bVelocityField =*/ true, /*bDampVelocityRule =*/ false>(Dt, Offset, Range, MinParallelBatchSize);
							}
							else  // VeloctiyFields
							{
								PreIterationUpdate</*bForceRule =*/ false, /*bVelocityField =*/ true, /*bDampVelocityRule =*/ false>(Dt, Offset, Range, MinParallelBatchSize);
							}
						}
					}
					else   // No Velocity Fields
					{
						if (MGroupLocalDampings[ParticleGroupId] > (FSolverReal)0.)
						{
							if (MGroupForceRules[ParticleGroupId])  // VeloctiyFields, Damping, Forces
							{
								PreIterationUpdate</*bForceRule =*/ true, /*bVelocityField =*/ false, /*bDampVelocityRule =*/ true>(Dt, Offset, Range, MinParallelBatchSize);
							}
							else  // VeloctiyFields, Damping
							{
								PreIterationUpdate</*bForceRule =*/ false, /*bVelocityField =*/ false, /*bDampVelocityRule =*/ true>(Dt, Offset, Range, MinParallelBatchSize);
							}
						}
						else  // No Damping
						{
							if (MGroupForceRules[ParticleGroupId])  // VeloctiyFields, Forces
							{
								PreIterationUpdate</*bForceRule =*/ true, /*bVelocityField =*/ false, /*bDampVelocityRule =*/ false>(Dt, Offset, Range, MinParallelBatchSize);
							}
							else  // VeloctiyFields
							{
								PreIterationUpdate</*bForceRule =*/ false, /*bVelocityField =*/ false, /*bDampVelocityRule =*/ false>(Dt, Offset, Range, MinParallelBatchSize);
							}
						}
					}
				}, bUseSingleThreadedRange);
		}

		// Collision update
		{
			if (MCollisionKinematicUpdate)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(ChaosNewtonCollisionKinematicUpdate);
				//SCOPE_CYCLE_COUNTER(STAT_ChaosNewtonCollisionKinematicUpdate);

				MCollisionParticlesActiveView.SequentialFor(
					[this, Dt](FSolverCollisionParticles& CollisionParticles, int32 Index)
					{
						// Store active collision particle frames prior to the kinematic update for CCD collisions
						MCollisionTransforms[Index] = FSolverRigidTransform3(CollisionParticles.GetX(Index), CollisionParticles.GetR(Index));

						// Update collision transform and velocity
						MCollisionKinematicUpdate(CollisionParticles, Dt, MTime, Index);
					});
			}

			{
				TRACE_CPUPROFILER_EVENT_SCOPE(ChaosNewtonClearCollidedArray);
				//SCOPE_CYCLE_COUNTER(STAT_ChaosNewtonClearCollidedArray);
				memset(MCollided.GetData(), 0, MCollided.Num() * sizeof(bool));
			}
		}

		// Constraint init (clear XNewton's Lambdas, init self collisions)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ChaosXNewtonConstraintsInit);
			//SCOPE_CYCLE_COUNTER(STAT_ChaosXNewtonConstraintsInit);
			MConstraintInitsActiveView.SequentialFor(
				[this, Dt](TArray<TFunction<void(FSolverParticles&, const FSolverReal)>>& ConstraintInits, int32 Index)
				{
					ConstraintInits[Index](MParticles, Dt);
				});
		}

		// Collision rule initializations
		MCollisionContacts.Reset();
		MCollisionNormals.Reset();

		// Iteration loop
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ChaosNewtonIterationLoop);
			//SCOPE_CYCLE_COUNTER(STAT_ChaosNewtonIterationLoop);


			DoNewtonStep(MNumNewtonIterations, MNewtonTol, MNumCGIterations, MCGTol, PStress, DeltaPStress, MTime, Dt, MIncidentElements, MParticles.GetM(), SetBCs, ProjectBCs, MAddExternalForce, UpdatePositionBasedState, MParticles, MResidualNorm, true, NoVerbose);


			{
				//SCOPE_CYCLE_COUNTER(STAT_ChaosNewtonPostIterationUpdates);

				// Particle update, V = (P - X) / Dt; X = P;

				{
					MParticlesActiveView.ParallelFor(
						[Dt, this](FSolverParticles& Particles, int32 Index)
						{
							Particles.SetV(Index, (Particles.P(Index) - Particles.GetX(Index)) / Dt);
							Particles.SetX(Index, Particles.P(Index));
							this->MVn[Index] = Particles.GetV(Index);
						}, MinParallelBatchSize);
				}
			}
		}

		// The following is not currently been used by the cloth solver implementation at the moment
		//if (!CVarChaosNewtonEvolutionFastPositionBasedFriction.GetValueOnAnyThread() && MCoefficientOfFriction > 0)
		//{
		//	SCOPE_CYCLE_COUNTER(STAT_ChaosNewtonCollisionRuleFriction);
		//	MParticlesActiveView.ParallelFor(
		//		[&CollisionRule, Dt](FSolverParticles& Particles, int32 Index)
		//		{
		//			TRACE_CPUPROFILER_EVENT_SCOPE(ChaosNewtonCollisionRuleFriction);
		//			CollisionRule.ApplyFriction(Particles, Dt, Index);
		//		}, bUseSingleThreadedRange, MinParallelBatchSize);
		//}
	}

}  // End namespace Chaos::Softs
