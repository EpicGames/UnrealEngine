// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/OBBVectorized.h"
#include "Chaos/Core.h"
#include "Chaos/Transform.h"
#include "Chaos/AABBVectorized.h"

namespace Chaos
{

	Private::FOBBVectorized::FOBBVectorized(const FRigidTransform3& Transform, const FVec3f& HalfExtentsIn, const FVec3f& InvScaleIn)
	{
		TRotation<FRealSingle, 3> Rotation = Transform.GetRotation();
		PMatrix<FRealSingle, 3, 3> Matrix = Rotation.ToMatrix();
		XAxis = VectorLoadFloat3(Matrix.M[0]);
		YAxis = VectorLoadFloat3(Matrix.M[1]);
		ZAxis = VectorLoadFloat3(Matrix.M[2]);

		FVec3 Translation = Transform.GetTranslation();
		Position = MakeVectorRegisterFloatFromDouble(VectorLoadDouble3(&Translation.X));
		InvScale = VectorLoadFloat3(&InvScaleIn.X);
		Position = VectorMultiply(Position, InvScale);

		HalfExtents = VectorLoadFloat3(&HalfExtentsIn.X);
		HalfExtents = VectorAdd(HalfExtents, GlobalVectorConstants::KindaSmallNumber);

		const VectorRegister4Float XHalfExtent = VectorReplicate(HalfExtents, 0);
		const VectorRegister4Float YHalfExtent = VectorReplicate(HalfExtents, 1);
		const VectorRegister4Float ZHalfExtent = VectorReplicate(HalfExtents, 2);

		VectorRegister4Float FurthestPoint = VectorMultiplyAdd(VectorAbs(XAxis), XHalfExtent, VectorMultiplyAdd(VectorAbs(YAxis), YHalfExtent, VectorMultiply(VectorAbs(ZAxis), ZHalfExtent)));

		FurthestPoint = VectorMultiply(FurthestPoint, VectorAbs(InvScale));
		MaxObb = VectorAdd(Position, FurthestPoint);
		MinObb = VectorSubtract(Position, FurthestPoint);

		const VectorRegister4Float Scale = VectorDivide(GlobalVectorConstants::FloatOne, InvScale);

		XAxis = VectorMultiply(XAxis, Scale);
		YAxis = VectorMultiply(YAxis, Scale);
		ZAxis = VectorMultiply(ZAxis, Scale);
	}

	bool Private::FOBBVectorized::IntersectAABB(const FAABBVectorized& Bounds) const
	{
		const VectorRegister4Float HasSeparationAxis = VectorBitwiseOr(VectorCompareGT(MinObb, Bounds.GetMax()), VectorCompareGT(Bounds.GetMin(), MaxObb));

		if (VectorMaskBits(HasSeparationAxis) != 0)
		{
			return false;
		}

		const VectorRegister4Float HalfAabb = VectorMultiply(VectorSubtract(Bounds.GetMax(), Bounds.GetMin()), GlobalVectorConstants::FloatOneHalf);
		const VectorRegister4Float CenterAabb = VectorMultiply(VectorAdd(Bounds.GetMax(), Bounds.GetMin()), GlobalVectorConstants::FloatOneHalf);
		VectorRegister4Float DirCenter = VectorSubtract(CenterAabb, Position);


		const VectorRegister4Float DistCenterX = VectorAbs(VectorDot3(XAxis, DirCenter));
		const VectorRegister4Float DistCenterY = VectorAbs(VectorDot3(YAxis, DirCenter));
		const VectorRegister4Float DistCenterZ = VectorAbs(VectorDot3(ZAxis, DirCenter));

		VectorRegister4Float DistCenter = VectorUnpackLo(DistCenterX, DistCenterY);
		DistCenter = VectorMoveLh(DistCenter, DistCenterZ);

		const VectorRegister4Float DistCornerX = VectorDot3(VectorAbs(XAxis), HalfAabb);
		const VectorRegister4Float DistCornerY = VectorDot3(VectorAbs(YAxis), HalfAabb);
		const VectorRegister4Float DistCornerZ = VectorDot3(VectorAbs(ZAxis), HalfAabb);

		VectorRegister4Float DistCorner = VectorUnpackLo(DistCornerX, DistCornerY);
		DistCorner = VectorMoveLh(DistCorner, DistCornerZ);

		const uint32 MaskBitTrue = VectorMaskBits(VectorCompareGT(VectorSubtract(DistCenter, DistCorner), HalfExtents));

		return !(MaskBitTrue & 7);
	}
}