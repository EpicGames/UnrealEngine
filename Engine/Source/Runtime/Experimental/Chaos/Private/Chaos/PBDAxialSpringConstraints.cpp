// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/PBDAxialSpringConstraints.h"
#include "Chaos/GraphColoring.h"
#include "Chaos/Framework/Parallel.h"
#include "Chaos/TriangleMesh.h"
#include "ChaosStats.h"
#if INTEL_ISPC
#include "PBDAxialSpringConstraints.ispc.generated.h"
#endif

DECLARE_CYCLE_STAT(TEXT("Chaos PBD Axial Spring Constraint"), STAT_PBD_AxialSpring, STATGROUP_Chaos);

#if INTEL_ISPC
#if !UE_BUILD_SHIPPING || USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING
bool bChaos_AxialSpring_ISPC_Enabled = CHAOS_AXIAL_SPRING_ISPC_ENABLED_DEFAULT;
FAutoConsoleVariableRef CVarChaosAxialSpringISPCEnabled(TEXT("p.Chaos.AxialSpring.ISPC"), bChaos_AxialSpring_ISPC_Enabled, TEXT("Whether to use ISPC optimizations in AxialSpring constraints"));
#endif

static_assert(sizeof(ispc::FVector4f) == sizeof(Chaos::Softs::FPAndInvM), "sizeof(ispc::FVector4f) != sizeof(Chaos::Softs::FPAndInvM");
static_assert(sizeof(ispc::FIntVector) == sizeof(Chaos::TVec3<int32>), "sizeof(ispc::FIntVector) != sizeof(Chaos::TVec3<int32>");
#endif

namespace Chaos::Softs {

// @todo(chaos): the parallel threshold (or decision to run parallel) should probably be owned by the solver and passed to the constraint container
int32 Chaos_AxialSpring_ParallelConstraintCount = 100;
FAutoConsoleVariableRef CVarChaosAxialSpringParallelConstraintCount(TEXT("p.Chaos.AxialSpring.ParallelConstraintCount"), Chaos_AxialSpring_ParallelConstraintCount, TEXT("If we have more constraints than this, use parallel-for in Apply."));

template<typename SolverParticlesOrRange>
TArray<int32> FPBDAxialSpringConstraints::InitColor(const SolverParticlesOrRange& InParticles)
{
	TArray<int32> OrigToReorderedIndices; // used to reorder stiffness indices

	// In dev builds we always color so we can tune the system without restarting. See Apply()
#if UE_BUILD_SHIPPING || UE_BUILD_TEST
	if (Constraints.Num() > Chaos_AxialSpring_ParallelConstraintCount)
#endif
	{
		const TArray<TArray<int32>> ConstraintsPerColor = FGraphColoring::ComputeGraphColoringParticlesOrRange(Constraints, InParticles, ParticleOffset, ParticleOffset + ParticleCount);

		// Reorder constraints based on color so each array in ConstraintsPerColor contains contiguous elements.
		TArray<TVec3<int32>> ReorderedConstraints;
		TArray<FSolverReal> ReorderedBarys;
		TArray<FSolverReal> ReorderedDists;
		ReorderedConstraints.SetNumUninitialized(Constraints.Num());
		ReorderedBarys.SetNumUninitialized(Constraints.Num());
		ReorderedDists.SetNumUninitialized(Constraints.Num());
		OrigToReorderedIndices.SetNumUninitialized(Constraints.Num());

		ConstraintsPerColorStartIndex.Reset(ConstraintsPerColor.Num() + 1);

		int32 ReorderedIndex = 0;
		for (const TArray<int32>& ConstraintsBatch : ConstraintsPerColor)
		{
			ConstraintsPerColorStartIndex.Add(ReorderedIndex);
			for (const int32& BatchConstraint : ConstraintsBatch)
			{
				const int32 OrigIndex = BatchConstraint;
				ReorderedConstraints[ReorderedIndex] = Constraints[OrigIndex];
				ReorderedBarys[ReorderedIndex] = Barys[OrigIndex];
				ReorderedDists[ReorderedIndex] = Dists[OrigIndex];
				OrigToReorderedIndices[OrigIndex] = ReorderedIndex;
				++ReorderedIndex;
			}
		}
		ConstraintsPerColorStartIndex.Add(ReorderedIndex);

		Constraints = MoveTemp(ReorderedConstraints);
		Barys = MoveTemp(ReorderedBarys);
		Dists = MoveTemp(ReorderedDists);
		Stiffness.ReorderIndices(OrigToReorderedIndices);
	}
	return OrigToReorderedIndices;
}
template CHAOS_API TArray<int32> FPBDAxialSpringConstraints::InitColor(const FSolverParticles& InParticles);
template CHAOS_API TArray<int32> FPBDAxialSpringConstraints::InitColor(const FSolverParticlesRange& InParticles);

template<typename SolverParticlesOrRange>
void FPBDAxialSpringConstraints::ApplyHelper(SolverParticlesOrRange& Particles, const FSolverReal Dt, const int32 ConstraintIndex, const FSolverReal ExpStiffnessValue) const
{
		const TVec3<int32>& Constraint = Constraints[ConstraintIndex];
		const int32 i1 = Constraint[0];
		const int32 i2 = Constraint[1];
		const int32 i3 = Constraint[2];
		const FSolverVec3 Delta = Base::GetDelta(Particles, ConstraintIndex, ExpStiffnessValue);
		const FSolverReal Multiplier = (FSolverReal)2. / (FMath::Max(Barys[ConstraintIndex], (FSolverReal)1. - Barys[ConstraintIndex]) + (FSolverReal)1.);
		if (Particles.InvM(i1) > (FSolverReal)0.)
		{
			Particles.P(i1) -= Multiplier * Particles.InvM(i1) * Delta;
		}
		if (Particles.InvM(i2) > (FSolverReal)0.)
		{
			Particles.P(i2) += Multiplier * Particles.InvM(i2) * Barys[ConstraintIndex] * Delta;
		}
		if (Particles.InvM(i3) > (FSolverReal)0.)
		{
			Particles.P(i3) += Multiplier * Particles.InvM(i3) * ((FSolverReal)1. - Barys[ConstraintIndex]) * Delta;
		}
}
template void FPBDAxialSpringConstraints::ApplyHelper(FSolverParticles& Particles, const FSolverReal Dt, const int32 ConstraintIndex, const FSolverReal ExpStiffnessValue) const;
template void FPBDAxialSpringConstraints::ApplyHelper(FSolverParticlesRange& Particles, const FSolverReal Dt, const int32 ConstraintIndex, const FSolverReal ExpStiffnessValue) const;

template<typename SolverParticlesOrRange>
void FPBDAxialSpringConstraints::Apply(SolverParticlesOrRange& Particles, const FSolverReal Dt) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPBDAxialSpringConstraints_Apply);
	SCOPE_CYCLE_COUNTER(STAT_PBD_AxialSpring);
	if (ConstraintsPerColorStartIndex.Num() > 0 && Constraints.Num() > Chaos_AxialSpring_ParallelConstraintCount)
	{
		const int32 ConstraintColorNum = ConstraintsPerColorStartIndex.Num() - 1;
		if (!Stiffness.HasWeightMap())
		{
			const FSolverReal ExpStiffnessValue = (FSolverReal)Stiffness;

#if INTEL_ISPC
			if (bRealTypeCompatibleWithISPC && bChaos_AxialSpring_ISPC_Enabled)
			{
				for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
				{
					const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
					const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
					ispc::ApplyAxialSpringConstraints(
						(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
						(ispc::FIntVector*)&Constraints.GetData()[ColorStart],
						&Barys.GetData()[ColorStart],
						&Dists.GetData()[ColorStart],
						ExpStiffnessValue,
						ColorSize);
				}
			}
			else
#endif
			{
				for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
				{
					const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
					const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
					PhysicsParallelFor(ColorSize, [this, &Particles, Dt, ColorStart, ExpStiffnessValue](const int32 Index)
					{
						const int32 ConstraintIndex = ColorStart + Index;
						ApplyHelper(Particles, Dt, ConstraintIndex, ExpStiffnessValue);
					});
				}
			}
		}
		else  // Has weight maps
		{
#if INTEL_ISPC
			if (bRealTypeCompatibleWithISPC && bChaos_AxialSpring_ISPC_Enabled)
			{
				for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
				{
					const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
					const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
					ispc::ApplyAxialSpringConstraintsWithWeightMaps(
						(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
						(ispc::FIntVector*)&Constraints.GetData()[ColorStart],
						&Barys.GetData()[ColorStart],
						&Dists.GetData()[ColorStart],
						&Stiffness.GetIndices().GetData()[ColorStart],
						&Stiffness.GetTable().GetData()[0],
						ColorSize);
				}
			}
			else
#endif
			{
				for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
				{
					const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
					const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
					PhysicsParallelFor(ColorSize, [this, &Particles, Dt, ColorStart](const int32 Index)
					{
						const int32 ConstraintIndex = ColorStart + Index;
						const FSolverReal ExpStiffnessValue = Stiffness[ConstraintIndex];
						ApplyHelper(Particles, Dt, ConstraintIndex, ExpStiffnessValue);
					});
				}
			}
		}
	}
	else
	{
		if (!Stiffness.HasWeightMap())
		{
			const FSolverReal ExpStiffnessValue = (FSolverReal)Stiffness;
			for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
			{
				ApplyHelper(Particles, Dt, ConstraintIndex, ExpStiffnessValue);
			}
		}
		else
		{
			for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
			{
				const FSolverReal ExpStiffnessValue = Stiffness[ConstraintIndex];
				ApplyHelper(Particles, Dt, ConstraintIndex, ExpStiffnessValue);
			}
		}
	}
}
template CHAOS_API void FPBDAxialSpringConstraints::Apply(FSolverParticles& Particles, const FSolverReal Dt) const;
template CHAOS_API void FPBDAxialSpringConstraints::Apply(FSolverParticlesRange& Particles, const FSolverReal Dt) const;

FPBDAreaSpringConstraints::FPBDAreaSpringConstraints(
	const FSolverParticlesRange& Particles,
	const FTriangleMesh& TriangleMesh,
	const TArray<TVec3<FVec2f>>& FaceVertexPatternPositions,
	const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
	const FCollectionPropertyConstFacade& PropertyCollection,
	bool bTrimKinematicConstraints)
	: FPBDAxialSpringConstraints(
		Particles,
		TriangleMesh.GetElements(),
		WeightMaps.FindRef(GetAreaSpringStiffnessString(PropertyCollection, AreaSpringStiffnessName.ToString())),
		FSolverVec2(GetWeightedFloatAreaSpringStiffness(PropertyCollection, 1.f)),
		bTrimKinematicConstraints,
		false /*bInitColor*/
	)
	, WarpScale(FSolverVec2(GetWeightedFloatAreaSpringWarpScale(PropertyCollection, 1.f)),
		WeightMaps.FindRef(GetAreaSpringWarpScaleString(PropertyCollection, AreaSpringWarpScaleName.ToString())),
		TConstArrayView<TVec3<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, WeftScale(FSolverVec2(GetWeightedFloatAreaSpringWeftScale(PropertyCollection, 1.f)),
		WeightMaps.FindRef(GetAreaSpringWeftScaleString(PropertyCollection, AreaSpringWeftScaleName.ToString())),
		TConstArrayView<TVec3<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, AreaSpringStiffnessIndex(PropertyCollection)
	, AreaSpringWarpScaleIndex(PropertyCollection)
	, AreaSpringWeftScaleIndex(PropertyCollection)
{
	if (AreaSpringWarpScaleIndex != INDEX_NONE || AreaSpringWeftScaleIndex != INDEX_NONE)
	{
		InitFromPatternData(FaceVertexPatternPositions, TriangleMesh);
	}

	const TArray<int32> OrigToReorderedIndices = InitColor(Particles);
	if (bWarpWeftScaleEnabled && OrigToReorderedIndices.Num() == Constraints.Num())
	{
		TArray<FSolverReal> ReorderedBaseDists;
		TArray<FSolverVec2> ReorderedWarpWeftScaleBaseMultipliers;
		ReorderedBaseDists.SetNumUninitialized(Constraints.Num());
		ReorderedWarpWeftScaleBaseMultipliers.SetNumUninitialized(Constraints.Num());
		for (int32 OrigIndex = 0; OrigIndex < Constraints.Num(); ++OrigIndex)
		{
			const int32 ReorderedIndex = OrigToReorderedIndices[OrigIndex];
			ReorderedBaseDists[ReorderedIndex] = BaseDists[OrigIndex];
			ReorderedWarpWeftScaleBaseMultipliers[ReorderedIndex] = WarpWeftScaleBaseMultipliers[OrigIndex];
		}

		BaseDists = MoveTemp(ReorderedBaseDists);
		WarpWeftScaleBaseMultipliers = MoveTemp(ReorderedWarpWeftScaleBaseMultipliers);
		WarpScale.ReorderIndices(OrigToReorderedIndices);
		WeftScale.ReorderIndices(OrigToReorderedIndices);
	}
}

void FPBDAreaSpringConstraints::InitFromPatternData(const TArray<TVec3<FVec2f>>& FaceVertexPatternPositions, const FTriangleMesh& TriangleMesh)
{
	BaseDists = Dists; // Use Dists calculated by base class using 3D positions

	WarpWeftScaleBaseMultipliers.SetNumUninitialized(Constraints.Num());

	TConstArrayView<TArray<int32>> PointToTriangleMap = TriangleMesh.GetPointToTriangleMap();
	auto SortedElement = [](const TVec3<int32>& Element)
		{
			return TVec3<int32>(Element.Min(), Element.Mid(), Element.Max());
		};
	const TArray<TVec3<int32>>& Elements = TriangleMesh.GetElements();

	for (int32 ConstraintIdx = 0; ConstraintIdx < Constraints.Num(); ++ConstraintIdx)
	{
		const TVec3<int32>& Constraint = Constraints[ConstraintIdx];
		// Find corresponding triangle in original mesh (constraints get reordered by Base class init)
		const TArray<int32>& PossibleTriangles = PointToTriangleMap[Constraint[0]];
		const TVec3<int32> SortedConstraint = SortedElement(Constraint);
		int32 TriangleIndex = INDEX_NONE;
		for (const int32 Triangle : PossibleTriangles)
		{
			if (SortedConstraint == SortedElement(Elements[Triangle]))
			{
				TriangleIndex = Triangle;
				break;
			}
		}
		check(TriangleIndex != INDEX_NONE);

		const TVec3<int32>& Element = Elements[TriangleIndex];
		auto MatchIndex = [&Constraint, &Element](const int32 ConstraintAxis)
			{
				check(Element[0] == Constraint[ConstraintAxis] || Element[1] == Constraint[ConstraintAxis] || Element[2] == Constraint[ConstraintAxis]);
				return Element[0] == Constraint[ConstraintAxis] ? 0 : Element[1] == Constraint[ConstraintAxis] ? 1 : 2;
			};
		const int32 FaceIndex1 = MatchIndex(0);
		const int32 FaceIndex2 = MatchIndex(1);
		const int32 FaceIndex3 = MatchIndex(2);

		const FVec2f& UV1 = FaceVertexPatternPositions[TriangleIndex][FaceIndex1];
		const FVec2f& UV2 = FaceVertexPatternPositions[TriangleIndex][FaceIndex2];
		const FVec2f& UV3 = FaceVertexPatternPositions[TriangleIndex][FaceIndex3];

		const FVec2f UV = (UV2 - UV3) * Barys[ConstraintIdx] + UV3;
		const FSolverVec2 UVDiff = UV - UV1;
		const FSolverVec2 UVDiffAbs = UVDiff.GetAbs();
		const FSolverReal UVLength = UVDiffAbs.Length();
		WarpWeftScaleBaseMultipliers[ConstraintIdx] = UVLength > UE_SMALL_NUMBER ? UVDiffAbs / UVLength : FSolverVec2(UE_INV_SQRT_2, UE_INV_SQRT_2); // Default to equally scaling warp and weft directions if zero length
	}

	bWarpWeftScaleEnabled = true;
}

void FPBDAreaSpringConstraints::SetProperties(
	const FCollectionPropertyConstFacade& PropertyCollection,
	const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps)
{
	if (IsAreaSpringStiffnessMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedValue(GetWeightedFloatAreaSpringStiffness(PropertyCollection));
		if (IsAreaSpringStiffnessStringDirty(PropertyCollection))
		{
			const FString& WeightMapName = GetAreaSpringStiffnessString(PropertyCollection);
			Stiffness = FPBDStiffness(
				WeightedValue,
				WeightMaps.FindRef(WeightMapName),
				TConstArrayView<TVec3<int32>>(Constraints),
				ParticleOffset,
				ParticleCount);
		}
		else
		{
			Stiffness.SetWeightedValue(WeightedValue);
		}
	}
	if (bWarpWeftScaleEnabled)
	{
		if (IsAreaSpringWarpScaleMutable(PropertyCollection))
		{
			const FSolverVec2 WeightedValue(GetWeightedFloatAreaSpringWarpScale(PropertyCollection));
			if (IsAreaSpringWarpScaleStringDirty(PropertyCollection))
			{
				const FString& WeightMapName = GetAreaSpringWarpScaleString(PropertyCollection);
				WarpScale = FPBDWeightMap(
					WeightedValue,
					WeightMaps.FindRef(WeightMapName),
					TConstArrayView<TVec3<int32>>(Constraints),
					ParticleOffset,
					ParticleCount);
			}
			else
			{
				WarpScale.SetWeightedValue(WeightedValue);
			}
		}
		if (IsAreaSpringWeftScaleMutable(PropertyCollection))
		{
			const FSolverVec2 WeightedValue(GetWeightedFloatAreaSpringWeftScale(PropertyCollection));
			if (IsAreaSpringWeftScaleStringDirty(PropertyCollection))
			{
				const FString& WeightMapName = GetAreaSpringWeftScaleString(PropertyCollection);
				WeftScale = FPBDWeightMap(
					WeightedValue,
					WeightMaps.FindRef(WeightMapName),
					TConstArrayView<TVec3<int32>>(Constraints),
					ParticleOffset,
					ParticleCount);
			}
			else
			{
				WeftScale.SetWeightedValue(WeightedValue);
			}
		}
	}
}

void FPBDAreaSpringConstraints::ApplyProperties(const FSolverReal Dt, const int32 NumIterations)
{
	FPBDAxialSpringConstraints::ApplyProperties(Dt, NumIterations);

	if (bWarpWeftScaleEnabled)
	{
		bool bWarpScaleChanged = false;
		WarpScale.ApplyValues(&bWarpScaleChanged);
		bool bWeftScaleChanged = false;
		WeftScale.ApplyValues(&bWeftScaleChanged);
		if (bWarpScaleChanged || bWeftScaleChanged)
		{
			// Need to update distances
			UpdateDists();
		}
	}
}

void FPBDAreaSpringConstraints::UpdateDists()
{
	if (bWarpWeftScaleEnabled)
	{
		const bool WarpScaleHasWeightMap = WarpScale.HasWeightMap();
		const bool WeftScaleHasWeightMap = WeftScale.HasWeightMap();
		const FSolverReal WarpScaleNoMap = (FSolverReal)WarpScale;
		const FSolverReal WeftScaleNoMap = (FSolverReal)WeftScale;
		for (int32 ConstraintIndex = 0; ConstraintIndex < BaseDists.Num(); ++ConstraintIndex)
		{
			const FSolverReal WarpScaleValue = WarpScaleHasWeightMap ? WarpScale[ConstraintIndex] : WarpScaleNoMap;
			const FSolverReal WeftScaleValue = WeftScaleHasWeightMap ? WeftScale[ConstraintIndex] : WeftScaleNoMap;

			Dists[ConstraintIndex] = BaseDists[ConstraintIndex] * FMath::Sqrt(FMath::Square(WeftScaleValue * WarpWeftScaleBaseMultipliers[ConstraintIndex][0]) + FMath::Square(WarpScaleValue * WarpWeftScaleBaseMultipliers[ConstraintIndex][1]));
		}
	}
}

}  // End namespace Chaos::Softs
