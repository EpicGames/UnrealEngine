// Copyright Epic Games, Inc. All Rights Reserved.

#define EXPLICIT_VECTOR4 1

#include "Math/Vector.isph"
#include "Chaos/PBDSofts.isph"

export void ApplyAxialSpringConstraints(uniform FVector4f PandInvM[],
										const uniform FIntVector Constraints[],
										const uniform float Barys[],
										const uniform float Dists[],
										const uniform float Stiffness,
										const uniform int32 NumConstraints)
{

	foreach(i = 0 ... NumConstraints)
	{
		const varying FIntVector Constraint = VectorLoad(&Constraints[extract(i,0)]);

		const varying float Bary = Barys[i];
		const varying float Dist = Dists[i];

		const varying int32 i1 = Constraint.V[0];
		const varying int32 i2 = Constraint.V[1];
		const varying int32 i3 = Constraint.V[2];

		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[i2]);
		const varying FVector4f PandInvM3 = VectorGather(&PandInvM[i3]);

		varying FVector3f P1, P2, P3;
		varying float M1, M2, M3;
		UnzipPandInvM(PandInvM2, P2, M2);
		UnzipPandInvM(PandInvM3, P3, M3);

		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[i1]);
		UnzipPandInvM(PandInvM1, P1, M1);

		const FVector3f P0 = (P2 - P3) * Bary + P3;
		const FVector3f Difference = P1 - P0;

		const float Distance = VectorSize(Difference);
		const float Multiplier = 2 / (max(Bary, 1 - Bary) + 1);

		const FVector3f Direction = Difference / Distance;
		const FVector3f InnerDelta = (Distance - Dist) * Direction;

		const float PInvMass = M3 * (1 - Bary) + M2 * Bary;
		const float CombinedInvMass = PInvMass + M1;
		const FVector3f ComputedDelta = Stiffness * InnerDelta / CombinedInvMass;

		const FVector3f Delta = VectorSelect(((M1 == 0 && PInvMass == 0) || Distance <= FLOAT_SMALL_NUMBER), FloatZeroVector, ComputedDelta);

		if (M1 > 0)
		{
			VectorScatter(&PandInvM[i1], SetVector4(P1 - (Multiplier * M1 * Delta), M1));
		}
		if (M2)
		{
			VectorScatter(&PandInvM[i2], SetVector4(P2 + (Multiplier * M2 * Bary * Delta), M2));
		}
		if (M3)
		{
			VectorScatter(&PandInvM[i3], SetVector4(P3 + (Multiplier * M3 * (1 - Bary) * Delta), M3));
		}
	}
}

export void ApplyAxialSpringConstraintsWithWeightMaps(uniform FVector4f PandInvM[],
										const uniform FIntVector Constraints[],
										const uniform float Barys[],
										const uniform float Dists[],
										const uniform uint8 StiffnessIndices[],
										const uniform float StiffnessTable[],
										const uniform int32 NumConstraints)
{

	foreach(i = 0 ... NumConstraints)
	{
		const varying FIntVector Constraint = VectorLoad(&Constraints[extract(i,0)]);

		const varying float Bary = Barys[i];
		const varying float Dist = Dists[i];
		const varying uint8 StiffnessIndex = StiffnessIndices[i];

		const varying int32 i1 = Constraint.V[0];
		const varying int32 i2 = Constraint.V[1];
		const varying int32 i3 = Constraint.V[2];

		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[i2]);
		const varying FVector4f PandInvM3 = VectorGather(&PandInvM[i3]);

		varying FVector3f P1, P2, P3;
		varying float M1, M2, M3;
		UnzipPandInvM(PandInvM2, P2, M2);
		UnzipPandInvM(PandInvM3, P3, M3);

		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[i1]);
		UnzipPandInvM(PandInvM1, P1, M1);

		const FVector3f P0 = (P2 - P3) * Bary + P3;
		const FVector3f Difference = P1 - P0;

		const float Distance = VectorSize(Difference);
		const float Multiplier = 2 / (max(Bary, 1 - Bary) + 1);

		#pragma ignore warning(perf)
		const varying float Stiffness = StiffnessTable[StiffnessIndex];

		const FVector3f Direction = Difference / Distance;
		const FVector3f InnerDelta = (Distance - Dist) * Direction;

		const float PInvMass = M3 * (1 - Bary) + M2 * Bary;
		const float CombinedInvMass = PInvMass + M1;
		const FVector3f ComputedDelta = Stiffness * InnerDelta / CombinedInvMass;

		const FVector3f Delta = VectorSelect(((M1 == 0 && PInvMass == 0) || Distance <= FLOAT_SMALL_NUMBER), FloatZeroVector, ComputedDelta);

		if (M1 > 0)
		{
			VectorScatter(&PandInvM[i1], SetVector4(P1 - (Multiplier * M1 * Delta), M1));
		}
		if (M2)
		{
			VectorScatter(&PandInvM[i2], SetVector4(P2 + (Multiplier * M2 * Bary * Delta), M2));
		}
		if (M3)
		{
			VectorScatter(&PandInvM[i3], SetVector4(P3 + (Multiplier * M3 * (1 - Bary) * Delta), M3));
		}
	}
}
