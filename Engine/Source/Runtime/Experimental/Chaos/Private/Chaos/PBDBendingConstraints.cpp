// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/PBDBendingConstraints.h"
#include "Chaos/GraphColoring.h"
#include "Chaos/Framework/Parallel.h"
#include "ChaosStats.h"
#if INTEL_ISPC
#include "PBDBendingConstraints.ispc.generated.h"
#endif

DECLARE_CYCLE_STAT(TEXT("Chaos PBD Bending Constraint"), STAT_PBD_Bending, STATGROUP_Chaos);

#if INTEL_ISPC
static_assert(sizeof(ispc::FVector4f) == sizeof(Chaos::Softs::FPAndInvM), "sizeof(ispc::FVector4f) != sizeof(Chaos::Softs::FPAndInvM");
static_assert(sizeof(ispc::FIntVector4) == sizeof(Chaos::TVec4<int32>), "sizeof(ispc::FIntVector4) != sizeof(Chaos::TVec4<int32>");
#endif

namespace Chaos::Softs {

// @todo(chaos): the parallel threshold (or decision to run parallel) should probably be owned by the solver and passed to the constraint container
int32 Chaos_Bending_ParallelConstraintCount = 100;
FAutoConsoleVariableRef CVarChaosBendingParallelConstraintCount(TEXT("p.Chaos.Bending.ParallelConstraintCount"), Chaos_Bending_ParallelConstraintCount, TEXT("If we have more constraints than this, use parallel-for in Apply."));

template<typename SolverParticlesOrRange>
void FPBDBendingConstraints::InitColor(const SolverParticlesOrRange& InParticles)
{
	// In dev builds we always color so we can tune the system without restarting. See Apply()
#if UE_BUILD_SHIPPING || UE_BUILD_TEST
	if (Constraints.Num() > Chaos_Bending_ParallelConstraintCount)
#endif
	{
		const TArray<TArray<int32>> ConstraintsPerColor = FGraphColoring::ComputeGraphColoringParticlesOrRange(Constraints, InParticles, ParticleOffset, ParticleOffset + ParticleCount);

		// Reorder constraints based on color so each array in ConstraintsPerColor contains contiguous elements.
		TArray<TVec4<int32>> ReorderedConstraints; 
		TArray<TVec2<int32>> ReorderedConstraintSharedEdges;
		TArray<FSolverReal> ReorderedRestAngles;
		TArray<int32> OrigToReorderedIndices; // used to reorder stiffness indices
		ReorderedConstraints.SetNumUninitialized(Constraints.Num());
		ReorderedConstraintSharedEdges.SetNumUninitialized(Constraints.Num());
		ReorderedRestAngles.SetNumUninitialized(Constraints.Num());
		OrigToReorderedIndices.SetNumUninitialized(Constraints.Num());

		ConstraintsPerColorStartIndex.Reset(ConstraintsPerColor.Num() + 1);

		int32 ReorderedIndex = 0;
		for (const TArray<int32>& ConstraintsBatch : ConstraintsPerColor)
		{
			ConstraintsPerColorStartIndex.Add(ReorderedIndex);
			for (const int32& BatchConstraint : ConstraintsBatch)
			{
				const int32 OrigIndex = BatchConstraint;
				ReorderedConstraints[ReorderedIndex] = Constraints[OrigIndex];
				ReorderedConstraintSharedEdges[ReorderedIndex] = ConstraintSharedEdges[OrigIndex];
				ReorderedRestAngles[ReorderedIndex] = RestAngles[OrigIndex];
				OrigToReorderedIndices[OrigIndex] = ReorderedIndex;
				++ReorderedIndex;
			}
		}
		ConstraintsPerColorStartIndex.Add(ReorderedIndex);

		Constraints = MoveTemp(ReorderedConstraints);
		ConstraintSharedEdges = MoveTemp(ReorderedConstraintSharedEdges);
		RestAngles = MoveTemp(ReorderedRestAngles);
		Stiffness.ReorderIndices(OrigToReorderedIndices);
		BucklingRatioWeighted.ReorderIndices(OrigToReorderedIndices);
		BucklingStiffness.ReorderIndices(OrigToReorderedIndices);
	}
}
template CHAOS_API void FPBDBendingConstraints::InitColor(const FSolverParticles& InParticles);
template CHAOS_API void FPBDBendingConstraints::InitColor(const FSolverParticlesRange& InParticles);

void FPBDBendingConstraints::SetProperties(
	const FCollectionPropertyConstFacade& PropertyCollection,
	const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps)
{
	if (IsBendingElementStiffnessMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedValue(GetWeightedFloatBendingElementStiffness(PropertyCollection));
		if (IsBendingElementStiffnessStringDirty(PropertyCollection))
		{
			const FString& WeightMapName = GetBendingElementStiffnessString(PropertyCollection);
			Stiffness = FPBDStiffness(
				WeightedValue,
				WeightMaps.FindRef(WeightMapName),
				TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
				ParticleOffset,
				ParticleCount);
		}
		else
		{
			Stiffness.SetWeightedValue(WeightedValue);
		}
	}
	if (IsBucklingRatioMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedValue = FSolverVec2(GetWeightedFloatBucklingRatio(PropertyCollection)).ClampAxes(0.f, 1.f);
		if (IsBucklingRatioStringDirty(PropertyCollection))
		{
			const FString& WeightMapName = GetBucklingRatioString(PropertyCollection);
			BucklingRatioWeighted = FPBDWeightMap(
				WeightedValue,
				WeightMaps.FindRef(WeightMapName),
				TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
				ParticleOffset,
				ParticleCount);
		}
		else
		{
			BucklingRatioWeighted.SetWeightedValue(WeightedValue);
		}
	}
	if (IsBucklingStiffnessMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedValue(GetWeightedFloatBucklingStiffness(PropertyCollection));
		if (IsBucklingStiffnessStringDirty(PropertyCollection))
		{
			const FString& WeightMapName = GetBucklingStiffnessString(PropertyCollection);
			BucklingStiffness = FPBDStiffness(
				WeightedValue,
				WeightMaps.FindRef(WeightMapName),
				TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
				ParticleOffset,
				ParticleCount);
		}
		else
		{
			BucklingStiffness.SetWeightedValue(WeightedValue);
		}
	}
}

template<typename SolverParticlesOrRange>
void FPBDBendingConstraints::ApplyHelper(SolverParticlesOrRange& Particles, const FSolverReal Dt, const int32 ConstraintIndex, const FSolverReal ExpStiffnessValue, const FSolverReal ExpBucklingValue) const
{

	const TVec4<int32>& Constraint = Constraints[ConstraintIndex];
	const int32 i1 = Constraint[0];
	const int32 i2 = Constraint[1];
	const int32 i3 = Constraint[2];
	const int32 i4 = Constraint[3];
	const TStaticArray<FSolverVec3, 4> Grads = Base::GetGradients(Particles, ConstraintIndex);
	const FSolverReal S = Base::GetScalingFactor(Particles, ConstraintIndex, Grads, ExpStiffnessValue, ExpBucklingValue);
	Particles.P(i1) -= S * Particles.InvM(i1) * Grads[0];
	Particles.P(i2) -= S * Particles.InvM(i2) * Grads[1];
	Particles.P(i3) -= S * Particles.InvM(i3) * Grads[2];
	Particles.P(i4) -= S * Particles.InvM(i4) * Grads[3];
}

template<typename SolverParticlesOrRange>
void FPBDBendingConstraints::Apply(SolverParticlesOrRange& Particles, const FSolverReal Dt) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPBDBendingConstraints_Apply);
	SCOPE_CYCLE_COUNTER(STAT_PBD_Bending);

	const bool StiffnessHasWeightMap = Stiffness.HasWeightMap();
	const bool BucklingStiffnessHasWeightMap = BucklingStiffness.HasWeightMap();

	if (ConstraintsPerColorStartIndex.Num() > 0 && Constraints.Num() > Chaos_Bending_ParallelConstraintCount)
	{
		const int32 ConstraintColorNum = ConstraintsPerColorStartIndex.Num() - 1;
		if (!StiffnessHasWeightMap && !BucklingStiffnessHasWeightMap)
		{
			const FSolverReal ExpStiffnessValue = (FSolverReal)Stiffness;
			const FSolverReal ExpBucklingValue = (FSolverReal)BucklingStiffness;
#if INTEL_ISPC
			if (bRealTypeCompatibleWithISPC && bChaos_Bending_ISPC_Enabled)
			{
				for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
				{
					const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
					const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
					ispc::ApplyBendingConstraints(
						(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
						(ispc::FIntVector4*)&Constraints.GetData()[ColorStart],
						&RestAngles.GetData()[ColorStart],
						&IsBuckled.GetData()[ColorStart],
						ExpStiffnessValue,
						ExpBucklingValue,
						ColorSize);
				}
			}
			else
#endif
			{
				for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
				{
					const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
					const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
					PhysicsParallelFor(ColorSize, [&](const int32 Index)
					{
						const int32 ConstraintIndex = ColorStart + Index;
						ApplyHelper(Particles, Dt, ConstraintIndex, ExpStiffnessValue, ExpBucklingValue);
					});
				}
			}
		}
		else  // Has weight maps
		{
#if INTEL_ISPC
			if (bRealTypeCompatibleWithISPC && bChaos_Bending_ISPC_Enabled)
			{
				for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
				{
					const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
					const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
					ispc::ApplyBendingConstraintsWithMaps(
						(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
						(ispc::FIntVector4*)&Constraints.GetData()[ColorStart],
						&RestAngles.GetData()[ColorStart],
						&IsBuckled.GetData()[ColorStart],
						StiffnessHasWeightMap,
						StiffnessHasWeightMap ? &Stiffness.GetIndices().GetData()[ColorStart] : nullptr,
						&Stiffness.GetTable().GetData()[0],
						BucklingStiffnessHasWeightMap,
						BucklingStiffnessHasWeightMap ? &BucklingStiffness.GetIndices().GetData()[ColorStart] : nullptr,
						&BucklingStiffness.GetTable().GetData()[0],
						ColorSize);
				}
			}
			else
#endif
			{
				const FSolverReal StiffnessNoMap = (FSolverReal)Stiffness;
				const FSolverReal BucklingStiffnessNoMap = (FSolverReal)BucklingStiffness;
				for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
				{
					const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
					const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
					PhysicsParallelFor(ColorSize, [&](const int32 Index)
					{
						const int32 ConstraintIndex = ColorStart + Index;
						const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? Stiffness[ConstraintIndex] : StiffnessNoMap;
						const FSolverReal ExpBucklingValue = BucklingStiffnessHasWeightMap ? BucklingStiffness[ConstraintIndex] : BucklingStiffnessNoMap;
						ApplyHelper(Particles, Dt, ConstraintIndex, ExpStiffnessValue, ExpBucklingValue);
					});
				}
			}
		}
	}
	else
	{
		if (!StiffnessHasWeightMap && !BucklingStiffnessHasWeightMap)
		{
			const FSolverReal ExpStiffnessValue = (FSolverReal)Stiffness;
			const FSolverReal ExpBucklingValue = (FSolverReal)BucklingStiffness;
			for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
			{
				ApplyHelper(Particles, Dt, ConstraintIndex, ExpStiffnessValue, ExpBucklingValue);
			}
		}
		else
		{
			const FSolverReal StiffnessNoMap = (FSolverReal)Stiffness;
			const FSolverReal BucklingStiffnessNoMap = (FSolverReal)BucklingStiffness;
			for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
			{
				const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? Stiffness[ConstraintIndex] : StiffnessNoMap;
				const FSolverReal ExpBucklingValue = BucklingStiffnessHasWeightMap ? BucklingStiffness[ConstraintIndex] : BucklingStiffnessNoMap;
				ApplyHelper(Particles, Dt, ConstraintIndex, ExpStiffnessValue, ExpBucklingValue);
			}
		}
	}
}
template CHAOS_API void FPBDBendingConstraints::Apply(FSolverParticles& Particles, const FSolverReal Dt) const;
template CHAOS_API void FPBDBendingConstraints::Apply(FSolverParticlesRange& Particles, const FSolverReal Dt) const;

}  // End namespace Chaos::Softs
