// Copyright Epic Games, Inc. All Rights Reserved.

#define EXPLICIT_VECTOR4 1

#include "Math/Vector.isph"
#include "Chaos/PBDSofts.isph"

static const uniform float SingleStepAngleLimit = FLOAT_PI * .25f;

static inline FVector3f SafeDivide(const FVector3f& Numerator, const float Denominator)
{
	return VectorSelect(Denominator <= FLOAT_SMALL_NUMBER, FloatZeroVector, Numerator / Denominator);
}

static inline varying float SafeDivide(const varying float Numerator, const varying float Denominator)
{
	return select(Denominator <= FLOAT_SMALL_NUMBER, (uniform float)0, Numerator / Denominator);
}

static inline float CalcAngle(const FVector3f& P1, const FVector3f& P2, const FVector3f& P3, const FVector3f& P4)
{
	const FVector3f SharedEdgeNormalized = VectorGetSafeNormal(P2 - P1);
	const FVector3f P13CrossP23 = VectorCross(P1 - P3, P2 - P3);
	const float Normal1Len = VectorSize(P13CrossP23);
	const FVector3f Normal1 = SafeDivide(P13CrossP23, Normal1Len);
	const FVector3f P24CrossP14 = VectorCross(P2 - P4, P1 - P4);
	const float Normal2Len = VectorSize(P24CrossP14);
	const FVector3f Normal2 = SafeDivide(P24CrossP14, Normal2Len);

	const FVector3f N2CrossN1 = VectorCross(Normal2, Normal1);

	const float CosPhi = clamp(VectorDot(Normal1, Normal2), (uniform float)(-1), (uniform float)(1));
	const float SinPhi = clamp(VectorDot(N2CrossN1, SharedEdgeNormalized), (uniform float)(-1), (uniform float)(1));
	return atan2(SinPhi, CosPhi);
}

// Returns Angle
static inline float CalcGradientsAndAngle(const FVector3f& P1, const FVector3f& P2, const FVector3f& P3, const FVector3f& P4, FVector3f& Grad1, FVector3f& Grad2, FVector3f& Grad3, FVector3f& Grad4)
{
	const FVector3f SharedEdgeNormalized = VectorGetSafeNormal(P2 - P1);
	const FVector3f P13CrossP23 = VectorCross(P1 - P3, P2 - P3);
	const float Normal1Len = VectorSize(P13CrossP23);
	const FVector3f Normal1 = SafeDivide(P13CrossP23, Normal1Len);
	const FVector3f P24CrossP14 = VectorCross(P2 - P4, P1 - P4);
	const float Normal2Len = VectorSize(P24CrossP14);
	const FVector3f Normal2 = SafeDivide(P24CrossP14, Normal2Len);

	const FVector3f N2CrossN1 = VectorCross(Normal2, Normal1);

	const float CosPhi = clamp(VectorDot(Normal1, Normal2), (uniform float)(-1), (uniform float)(1));
	const float SinPhi = clamp(VectorDot(N2CrossN1, SharedEdgeNormalized), (uniform float)(-1), (uniform float)(1));

	const FVector3f DPhiDN1_OverNormal1Len = SafeDivide(CosPhi * VectorCross(SharedEdgeNormalized, Normal2) - SinPhi * Normal2, Normal1Len);
	const FVector3f DPhiDN2_OverNormal2Len = SafeDivide(CosPhi * VectorCross(Normal1, SharedEdgeNormalized) - SinPhi * Normal1, Normal2Len);

	const FVector3f DPhiDP13 = VectorCross(P2 - P3, DPhiDN1_OverNormal1Len);
	const FVector3f DPhiDP23 = VectorCross(DPhiDN1_OverNormal1Len, P1 - P3);
	const FVector3f DPhiDP24 = VectorCross(P1 - P4, DPhiDN2_OverNormal2Len);
	const FVector3f DPhiDP14 = VectorCross(DPhiDN2_OverNormal2Len, P2 - P4);

	Grad1 = DPhiDP13 + DPhiDP14;
	Grad2 = DPhiDP23 + DPhiDP24;
	Grad3 = -1 * DPhiDP13 - DPhiDP23;
	Grad4 = -1 * DPhiDP14 - DPhiDP24;

	return atan2(SinPhi, CosPhi);
}

export void InitBendingConstraintsIsBuckled(const uniform FVector3f X[],
									const uniform FIntVector4 Constraints[],
									const uniform float RestAngles[],
									uniform bool IsBuckled[],
									const uniform float BucklingRatio,
									const uniform int32 NumConstraints)
{
	foreach(i = 0 ... NumConstraints)
	{
		const varying FIntVector4 Constraint = VectorLoad(&Constraints[extract(i,0)]);

		const varying int32 Index1 = Constraint.V[0];
		const varying int32 Index2 = Constraint.V[1];
		const varying int32 Index3 = Constraint.V[2];
		const varying int32 Index4 = Constraint.V[3];

		const varying FVector3f X1 = VectorGather(&X[Index1]);
		const varying FVector3f X2 = VectorGather(&X[Index2]);
		const varying FVector3f X3 = VectorGather(&X[Index3]);
		const varying FVector3f X4 = VectorGather(&X[Index4]);

		const varying float RestAngle = RestAngles[i];
		const varying float BucklingAngle = BucklingRatio * (FLOAT_PI - abs(RestAngle));

		const varying float Angle = CalcAngle(X1, X2, X3, X4);
		IsBuckled[i] = FLOAT_PI - abs(Angle) < BucklingAngle;
	}
}

export void InitBendingConstraintsIsBuckledWithMaps(const uniform FVector3f X[],
									const uniform FIntVector4 Constraints[],
									const uniform float RestAngles[],
									uniform bool IsBuckled[],
									const uniform uint8 BucklingRatioIndices[],
									const uniform float BucklingRatioTable[],
									const uniform int32 NumConstraints)
{
	foreach(i = 0 ... NumConstraints)
	{
		const varying FIntVector4 Constraint = VectorLoad(&Constraints[extract(i,0)]);

		const varying int32 Index1 = Constraint.V[0];
		const varying int32 Index2 = Constraint.V[1];
		const varying int32 Index3 = Constraint.V[2];
		const varying int32 Index4 = Constraint.V[3];
		const varying int8 BucklingRatioIndex = BucklingRatioIndices[i];

		const varying FVector3f X1 = VectorGather(&X[Index1]);
		const varying FVector3f X2 = VectorGather(&X[Index2]);
		const varying FVector3f X3 = VectorGather(&X[Index3]);
		const varying FVector3f X4 = VectorGather(&X[Index4]);
		
		#pragma ignore warning(perf)
		const varying float BucklingRatio = BucklingRatioTable[BucklingRatioIndex];
		const varying float RestAngle = RestAngles[i];
		const varying float BucklingAngle = BucklingRatio * (FLOAT_PI - abs(RestAngle));

		const varying float Angle = CalcAngle(X1, X2, X3, X4);
		IsBuckled[i] = FLOAT_PI - abs(Angle) < BucklingAngle;
	}
}

export void ApplyBendingConstraints(uniform FVector4f PandInvM[],
									const uniform FIntVector4 Constraints[],
									const uniform float RestAngles[],
									const uniform bool IsBuckled[],
									const uniform float Stiffness,
									const uniform float BucklingStiffness,
									const uniform int32 NumConstraints)
{

	foreach(i = 0 ... NumConstraints)
	{
		const varying FIntVector4 Constraint = VectorLoad(&Constraints[extract(i,0)]);

		const varying int32 i1 = Constraint.V[0];
		const varying int32 i2 = Constraint.V[1];
		const varying int32 i3 = Constraint.V[2];
		const varying int32 i4 = Constraint.V[3];

		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[i2]);
		const varying FVector4f PandInvM3 = VectorGather(&PandInvM[i3]);
		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[i1]);
		const varying FVector4f PandInvM4 = VectorGather(&PandInvM[i4]);

		varying FVector3f P1, P2, P3, P4;
		varying float M1, M2, M3, M4;

		UnzipPandInvM(PandInvM1, P1, M1);
		UnzipPandInvM(PandInvM2, P2, M2);
		UnzipPandInvM(PandInvM3, P3, M3);
		UnzipPandInvM(PandInvM4, P4, M4);


		varying FVector3f Grad1, Grad2, Grad3, Grad4;
		const varying float Angle = CalcGradientsAndAngle(P1, P2, P3, P4, Grad1, Grad2, Grad3, Grad4);

		const varying float Denom = M1 * VectorSizeSquared(Grad1) + M2 * VectorSizeSquared(Grad2) + M3 * VectorSizeSquared(Grad3) + M4 * VectorSizeSquared(Grad4);
		
		const varying float RestAngle = RestAngles[i];
		const varying float StiffnessValue = select(IsBuckled[i], BucklingStiffness, Stiffness);

		const varying float Delta = clamp(StiffnessValue * (Angle - RestAngle), -SingleStepAngleLimit, SingleStepAngleLimit);

		const varying float S = SafeDivide(Delta, Denom);

		if (M1 > 0)
		{
			VectorScatter(&PandInvM[i1], SetVector4(P1 - S * M1 * Grad1, M1));
		}
		if (M2 > 0)
		{
			VectorScatter(&PandInvM[i2], SetVector4(P2 - S * M2 * Grad2, M2));
		}
		if (M3 > 0)
		{
			VectorScatter(&PandInvM[i3], SetVector4(P3 - S * M3 * Grad3, M3));
		}
		if (M4 > 0)
		{
			VectorScatter(&PandInvM[i4], SetVector4(P4 - S * M4 * Grad4, M4));
		}
	}
}

export void ApplyBendingConstraintsWithMaps(uniform FVector4f PandInvM[],
											const uniform FIntVector4 Constraints[],
											const uniform float RestAngles[],
											const uniform bool IsBuckled[],
											const uniform bool StiffnessHasMap,
											const uniform uint8 StiffnessIndices[],
											const uniform float StiffnessTable[],
											const uniform bool BucklingStiffnessHasMap,
											const uniform uint8 BucklingStiffnessIndices[],
											const uniform float BucklingStiffnessTable[],
											const uniform int32 NumConstraints)
{

	foreach(i = 0 ... NumConstraints)
	{
		const varying FIntVector4 Constraint = VectorLoad(&Constraints[extract(i,0)]);

		const varying int32 i1 = Constraint.V[0];
		const varying int32 i2 = Constraint.V[1];
		const varying int32 i3 = Constraint.V[2];
		const varying int32 i4 = Constraint.V[3];
		const varying int8 StiffnessIndex = StiffnessHasMap ? StiffnessIndices[i] : 0;
		const varying int8 BucklingStiffnessIndex = BucklingStiffnessHasMap ? BucklingStiffnessIndices[i] : 0;

		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[i2]);
		const varying FVector4f PandInvM3 = VectorGather(&PandInvM[i3]);
		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[i1]);
		const varying FVector4f PandInvM4 = VectorGather(&PandInvM[i4]);

		varying FVector3f P1, P2, P3, P4;
		varying float M1, M2, M3, M4;

		UnzipPandInvM(PandInvM1, P1, M1);
		UnzipPandInvM(PandInvM2, P2, M2);
		UnzipPandInvM(PandInvM3, P3, M3);
		UnzipPandInvM(PandInvM4, P4, M4);


		varying FVector3f Grad1, Grad2, Grad3, Grad4;
		const varying float Angle = CalcGradientsAndAngle(P1, P2, P3, P4, Grad1, Grad2, Grad3, Grad4);

		const varying float Denom = M1 * VectorSizeSquared(Grad1) + M2 * VectorSizeSquared(Grad2) + M3 * VectorSizeSquared(Grad3) + M4 * VectorSizeSquared(Grad4);
		
		#pragma ignore warning(perf)
		const varying float Stiffness = StiffnessTable[StiffnessIndex];
		#pragma ignore warning(perf)
		const varying float BucklingStiffness = BucklingStiffnessTable[BucklingStiffnessIndex];

		const varying float RestAngle = RestAngles[i];
		const varying float StiffnessValue = select(IsBuckled[i], BucklingStiffness, Stiffness);

		const varying float Delta = clamp(StiffnessValue * (Angle - RestAngle), -SingleStepAngleLimit, SingleStepAngleLimit);

		const varying float S = SafeDivide(Delta, Denom);

		if (M1 > 0)
		{
			VectorScatter(&PandInvM[i1], SetVector4(P1 - S * M1 * Grad1, M1));
		}
		if (M2 > 0)
		{
			VectorScatter(&PandInvM[i2], SetVector4(P2 - S * M2 * Grad2, M2));
		}
		if (M3 > 0)
		{
			VectorScatter(&PandInvM[i3], SetVector4(P3 - S * M3 * Grad3, M3));
		}
		if (M4 > 0)
		{
			VectorScatter(&PandInvM[i4], SetVector4(P4 - S * M4 * Grad4, M4));
		}
	}
}
