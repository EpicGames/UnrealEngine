// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/PBDBendingConstraintsBase.h"
#if INTEL_ISPC
#include "PBDBendingConstraints.ispc.generated.h"
#endif

#if INTEL_ISPC 
#if !UE_BUILD_SHIPPING || USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING
bool bChaos_Bending_ISPC_Enabled = CHAOS_BENDING_ISPC_ENABLED_DEFAULT;
FAutoConsoleVariableRef CVarChaosBendingISPCEnabled(TEXT("p.Chaos.Bending.ISPC"), bChaos_Bending_ISPC_Enabled, TEXT("Whether to use ISPC optimizations in Bending constraints"));

#endif

static_assert(sizeof(ispc::FVector4f) == sizeof(Chaos::Softs::FPAndInvM), "sizeof(ispc::FVector4f) != sizeof(Chaos::Softs::FPAndInvM");
static_assert(sizeof(ispc::FIntVector4) == sizeof(Chaos::TVec4<int32>), "sizeof(ispc::FIntVector4) != sizeof(Chaos::TVec4<int32>");
#endif

namespace Chaos::Softs 
{
	namespace Private
	{
		void Calculate3DRestAngles(
			const TConstArrayView<FSolverVec3>& InPositions,
			const TArray<TVec4<int32>>& Constraints,
			TArray<FSolverReal>& RestAngles)
		{
			RestAngles.Reset(Constraints.Num());
			for (const TVec4<int32>& Constraint : Constraints)
			{
				const FSolverVec3& P1 = InPositions[Constraint[0]];
				const FSolverVec3& P2 = InPositions[Constraint[1]];
				const FSolverVec3& P3 = InPositions[Constraint[2]];
				const FSolverVec3& P4 = InPositions[Constraint[3]];
				RestAngles.Add(FMath::Clamp(FPBDBendingConstraintsBase::CalcAngle(P1, P2, P3, P4), -UE_PI, UE_PI));
			}
		}

		void CalculateFlatnessRestAngles(
			const TConstArrayView<FSolverVec3>& InPositions,
			int32 InParticleOffset,
			int32 InParticleCount,
			const TConstArrayView<FRealSingle>& RestAngleMap,
			const FSolverVec2& RestAngleValue,
			const TArray<TVec4<int32>>& Constraints,
			TArray<FSolverReal>& RestAngles)
		{
			if (RestAngleMap.Num() != InParticleCount && RestAngleValue[0] == 1.f)
			{
				// Special case where Flatness = 1, i.e., all RestAngles are 0.
				RestAngles.SetNumZeroed(Constraints.Num());
				return;
			}

			auto FlatnessValue = [InParticleOffset, InParticleCount, &RestAngleMap, &RestAngleValue](const TVec4<int32>& Constraint)->FSolverReal
			{
				if (RestAngleMap.Num() == InParticleCount)
				{
					return RestAngleValue[0] + (RestAngleValue[1] - RestAngleValue[0]) * 
						(RestAngleMap[Constraint[0] - InParticleOffset] + RestAngleMap[Constraint[1] - InParticleOffset]) * (FSolverReal).5f;
				}
				else
				{
					return RestAngleValue[0];
				}
			};

			RestAngles.Reset(Constraints.Num());
			for (const TVec4<int32>& Constraint : Constraints)
			{
				const FSolverVec3& P1 = InPositions[Constraint[0]];
				const FSolverVec3& P2 = InPositions[Constraint[1]];
				const FSolverVec3& P3 = InPositions[Constraint[2]];
				const FSolverVec3& P4 = InPositions[Constraint[3]];
				const FSolverReal Flatness = FMath::Clamp(FlatnessValue(Constraint), (FSolverReal)0.f, (FSolverReal)1.f);
				RestAngles.Add(FMath::Clamp(((FSolverReal)1.f - Flatness) * FPBDBendingConstraintsBase::CalcAngle(P1, P2, P3, P4), -UE_PI, UE_PI));
			}
		}

		void CalculateExplicitRestAngles(
			int32 InParticleOffset,
			int32 InParticleCount,
			const TConstArrayView<FRealSingle>& RestAngleMap,
			const FSolverVec2& RestAngleValue,
			const TArray<TVec4<int32>>& Constraints,
			TArray<FSolverReal>& RestAngles)
		{
			auto MapAngleRadians = [InParticleOffset, InParticleCount, &RestAngleMap, &RestAngleValue](const TVec4<int32>& Constraint)->FSolverReal
			{
				if (RestAngleMap.Num() == InParticleCount)
				{
					const FSolverReal RestAngle0 = FMath::UnwindRadians(FMath::DegreesToRadians(
						RestAngleValue[0] + (RestAngleValue[1] - RestAngleValue[0]) * (RestAngleMap[Constraint[0] - InParticleOffset])));
					const FSolverReal RestAngle1 = FMath::UnwindRadians(FMath::DegreesToRadians(
						RestAngleValue[0] + (RestAngleValue[1] - RestAngleValue[0]) * (RestAngleMap[Constraint[1] - InParticleOffset])));

					return FMath::Abs(RestAngle0) < FMath::Abs(RestAngle1) ? RestAngle0 : RestAngle1;
				}
				else
				{
					return FMath::UnwindRadians(FMath::DegreesToRadians(RestAngleValue[0]));
				}
			};

			RestAngles.Reset(Constraints.Num());
			for (const TVec4<int32>& Constraint : Constraints)
			{
				RestAngles.Add(FMath::Clamp(MapAngleRadians(Constraint), -UE_PI, UE_PI));
			}
		}
	}

	void FPBDBendingConstraintsBase::CalculateRestAngles(
		const TConstArrayView<FSolverVec3>& InPositions,
		int32 InParticleOffset,
		int32 InParticleCount,
		const TConstArrayView<FRealSingle>& RestAngleMap,
		const FSolverVec2& RestAngleValue,
		ERestAngleConstructionType RestAngleConstructionType)
	{
		switch (RestAngleConstructionType)
		{
		default:
		case ERestAngleConstructionType::Use3DRestAngles:
			Private::Calculate3DRestAngles(InPositions, Constraints, RestAngles);
			break;
		case ERestAngleConstructionType::FlatnessRatio:
			Private::CalculateFlatnessRestAngles(InPositions, InParticleOffset, InParticleCount, RestAngleMap, RestAngleValue, Constraints, RestAngles);
			break;
		case ERestAngleConstructionType::ExplicitRestAngles:
			Private::CalculateExplicitRestAngles(InParticleOffset, InParticleCount, RestAngleMap, RestAngleValue, Constraints, RestAngles);
		}
	}

	template<typename SolverParticlesOrRange>
	void FPBDBendingConstraintsBase::Init(const SolverParticlesOrRange& InParticles)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPBDBendingConstraintsBase_Init);

		IsBuckled.SetNumUninitialized(Constraints.Num());
#if INTEL_ISPC
		if (bRealTypeCompatibleWithISPC && bChaos_Bending_ISPC_Enabled)
		{
			if (BucklingRatioWeighted.HasWeightMap())
			{
				ispc::InitBendingConstraintsIsBuckledWithMaps(
					(const ispc::FVector3f*)InParticles.XArray().GetData(),
					(const ispc::FIntVector4*)Constraints.GetData(),
					RestAngles.GetData(),
					IsBuckled.GetData(),
					BucklingRatioWeighted.GetIndices().GetData(),
					BucklingRatioWeighted.GetTable().GetData(),
					Constraints.Num()
				);
			}
			else
			{
				ispc::InitBendingConstraintsIsBuckled(
					(const ispc::FVector3f*)InParticles.XArray().GetData(),
					(const ispc::FIntVector4*)Constraints.GetData(),
					RestAngles.GetData(),
					IsBuckled.GetData(),
					(FSolverReal)BucklingRatioWeighted,
					Constraints.Num()
				);
			}
		}
		else
#endif
		{
			for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
			{
				const TVec4<int32>& Constraint = Constraints[ConstraintIndex];
				const int32 Index1 = Constraint[0];
				const int32 Index2 = Constraint[1];
				const int32 Index3 = Constraint[2];
				const int32 Index4 = Constraint[3];
				const FSolverVec3& P1 = InParticles.X(Index1);
				const FSolverVec3& P2 = InParticles.X(Index2);
				const FSolverVec3& P3 = InParticles.X(Index3);
				const FSolverVec3& P4 = InParticles.X(Index4);
				const FSolverReal Angle = CalcAngle(P1, P2, P3, P4);
				IsBuckled[ConstraintIndex] = AngleIsBuckled(Angle, ConstraintIndex);
			}
		}
	}
	template CHAOS_API void FPBDBendingConstraintsBase::Init(const FSolverParticles& InParticles);
	template CHAOS_API void FPBDBendingConstraintsBase::Init(const FSolverParticlesRange& InParticles);
}  // End namespace Chaos::Softs
