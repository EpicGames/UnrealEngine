// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/PBDCollisionConstraintsUtil.h"

#include "Chaos/BoundingVolumeHierarchy.h"
#include "Chaos/Defines.h"
#include "Chaos/Pair.h"
#include "Chaos/Sphere.h"
#include "Chaos/Transform.h"
#include "ChaosStats.h"
#include "ChaosLog.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "Chaos/ImplicitObjectUnion.h" 

namespace Chaos
{
	void ComputeHashTable(const TArray<Chaos::FPBDCollisionConstraint>& ConstraintsArray,
						  const FBox& BoundingBox, TMultiMap<int32, int32>& HashTableMap, const FReal SpatialHashRadius)
	{
		FReal CellSize = 2 * SpatialHashRadius;
		check(CellSize > 0);

		// Compute number of cells along the principal axis
		FVector Extent = 2 * BoundingBox.GetExtent();
		FReal PrincipalAxisLength;
		if (Extent.X > Extent.Y && Extent.X > Extent.Z)
		{
			PrincipalAxisLength = Extent.X;
		}
		else if (Extent.Y > Extent.Z)
		{
			PrincipalAxisLength = Extent.Y;
		}
		else
		{
			PrincipalAxisLength = Extent.Z;
		}
		int32 NumberOfCells = FMath::CeilToInt32(PrincipalAxisLength / CellSize);
		check(NumberOfCells > 0);

		CellSize = PrincipalAxisLength / (FReal)NumberOfCells;
		FReal CellSizeInv = 1 / CellSize;

		int32 NumberOfCellsX = FMath::CeilToInt32(Extent.X * CellSizeInv) + 1;
		int32 NumberOfCellsY = FMath::CeilToInt32(Extent.Y * CellSizeInv) + 1;
		int32 NumberOfCellsZ = FMath::CeilToInt32(Extent.Z * CellSizeInv) + 1;

		// Create a Hash Table, but only store the buckets with constraint(s) as a map
		// HashTableMap<BucketIdx, ConstraintIdx>
		int32 NumberOfCellsXY = NumberOfCellsX * NumberOfCellsY;
		int32 NumberOfCellsXYZ = NumberOfCellsX * NumberOfCellsY * NumberOfCellsZ;
		for (int32 IdxConstraint = 0; IdxConstraint < ConstraintsArray.Num(); ++IdxConstraint)
		{
			FVector Location = (FVector)ConstraintsArray[IdxConstraint].CalculateWorldContactLocation() - BoundingBox.Min + FVector(0.5f * CellSize);
			int32 HashTableIdx = (int32)(Location.X * CellSizeInv) +
								 (int32)(Location.Y * CellSizeInv) * NumberOfCellsX +
								 (int32)(Location.Z * CellSizeInv) * NumberOfCellsXY;
			if (ensure(HashTableIdx < NumberOfCellsXYZ))
			{
				HashTableMap.Add(HashTableIdx, IdxConstraint);
			}
		}
	}

	void ComputeHashTable(const TArray<FCollidingData>& CollisionsArray,
						  const FBox& BoundingBox, TMultiMap<int32, int32>& HashTableMap, const FReal SpatialHashRadius)
	{
		FReal CellSize = 2 * SpatialHashRadius;
		check(CellSize > 0);

		// Compute number of cells along the principal axis
		FVector Extent = 2 * BoundingBox.GetExtent();
		FReal PrincipalAxisLength;
		if (Extent.X > Extent.Y && Extent.X > Extent.Z)
		{
			PrincipalAxisLength = Extent.X;
		}
		else if (Extent.Y > Extent.Z)
		{
			PrincipalAxisLength = Extent.Y;
		}
		else
		{
			PrincipalAxisLength = Extent.Z;
		}
		int32 NumberOfCells = FMath::CeilToInt32(PrincipalAxisLength / CellSize);
		check(NumberOfCells > 0);

		CellSize = PrincipalAxisLength / (FReal)NumberOfCells;
		FReal CellSizeInv = 1 / CellSize;

		int32 NumberOfCellsX = FMath::CeilToInt32(Extent.X * CellSizeInv) + 1;
		int32 NumberOfCellsY = FMath::CeilToInt32(Extent.Y * CellSizeInv) + 1;
		int32 NumberOfCellsZ = FMath::CeilToInt32(Extent.Z * CellSizeInv) + 1;

		// Create a Hash Table, but only store the buckets with constraint(s) as a map
		// HashTableMap<BucketIdx, ConstraintIdx>
		int32 NumberOfCellsXY = NumberOfCellsX * NumberOfCellsY;
		int32 NumberOfCellsXYZ = NumberOfCellsX * NumberOfCellsY * NumberOfCellsZ;
		for (int32 IdxCollision = 0; IdxCollision < CollisionsArray.Num(); ++IdxCollision)
		{
			FVector Location = (FVector)CollisionsArray[IdxCollision].Location - BoundingBox.Min + FVector(0.5f * CellSize);
			int32 HashTableIdx = (int32)(Location.X * CellSizeInv) +
								 (int32)(Location.Y * CellSizeInv) * NumberOfCellsX +
								 (int32)(Location.Z * CellSizeInv) * NumberOfCellsXY;
			if (ensure(HashTableIdx < NumberOfCellsXYZ))
			{
				HashTableMap.Add(HashTableIdx, IdxCollision);
			}
		}
	}

	void ComputeHashTable(const TArray<FCollidingDataExt>& CollisionsArray,
						  const FBox& BoundingBox, TMultiMap<int32, int32>& HashTableMap, const FReal SpatialHashRadius)
	{
		FReal CellSize = 2 * SpatialHashRadius;
		check(CellSize > 0);

		// Compute number of cells along the principal axis
		FVector Extent = 2 * BoundingBox.GetExtent();
		FReal PrincipalAxisLength;
		if (Extent.X > Extent.Y && Extent.X > Extent.Z)
		{
			PrincipalAxisLength = Extent.X;
		}
		else if (Extent.Y > Extent.Z)
		{
			PrincipalAxisLength = Extent.Y;
		}
		else
		{
			PrincipalAxisLength = Extent.Z;
		}
		int32 NumberOfCells = FMath::CeilToInt32(PrincipalAxisLength / CellSize);
		check(NumberOfCells > 0);

		CellSize = PrincipalAxisLength / (FReal)NumberOfCells;
		FReal CellSizeInv = 1 / CellSize;

		int32 NumberOfCellsX = FMath::CeilToInt32(Extent.X * CellSizeInv) + 1;
		int32 NumberOfCellsY = FMath::CeilToInt32(Extent.Y * CellSizeInv) + 1;
		int32 NumberOfCellsZ = FMath::CeilToInt32(Extent.Z * CellSizeInv) + 1;

		// Create a Hash Table, but only store the buckets with constraint(s) as a map
		// HashTableMap<BucketIdx, ConstraintIdx>
		int32 NumberOfCellsXY = NumberOfCellsX * NumberOfCellsY;
		int32 NumberOfCellsXYZ = NumberOfCellsXY * NumberOfCellsZ;
		// int overflow
		if (NumberOfCellsXYZ < 0)
		{
			CellSize = PrincipalAxisLength / 1000;
			CellSizeInv = 1 / CellSize;
			NumberOfCellsX = FMath::CeilToInt32(Extent.X * CellSizeInv) + 1;
			NumberOfCellsY = FMath::CeilToInt32(Extent.Y * CellSizeInv) + 1;
			NumberOfCellsZ = FMath::CeilToInt32(Extent.Z * CellSizeInv) + 1;
			NumberOfCellsXY = NumberOfCellsX * NumberOfCellsY;
			NumberOfCellsXYZ = NumberOfCellsXY * NumberOfCellsZ;
		}
		for (int32 IdxCollision = 0; IdxCollision < CollisionsArray.Num(); ++IdxCollision)
		{
			FVector Location = (FVector)CollisionsArray[IdxCollision].Location - BoundingBox.Min + FVector(0.5f * CellSize);
			int32 HashTableIdx = (int32)(Location.X * CellSizeInv) +
				(int32)(Location.Y * CellSizeInv) * NumberOfCellsX +
				(int32)(Location.Z * CellSizeInv) * NumberOfCellsXY;
			if (ensure(HashTableIdx < NumberOfCellsXYZ))
			{
				HashTableMap.Add(HashTableIdx, IdxCollision);
			}
		}
	}

	void ComputeHashTable(const TArray<FVector>& ParticleArray, const FBox& BoundingBox, TMultiMap<int32, int32>& HashTableMap, const FReal SpatialHashRadius)
	{
		FReal CellSize = 2 * SpatialHashRadius;
		check(CellSize > 0);

		// Compute number of cells along the principal axis
		FVector Extent = 2 * BoundingBox.GetExtent();
		FReal PrincipalAxisLength;
		if (Extent.X > Extent.Y && Extent.X > Extent.Z)
		{
			PrincipalAxisLength = Extent.X;
		}
		else if (Extent.Y > Extent.Z)
		{
			PrincipalAxisLength = Extent.Y;
		}
		else
		{
			PrincipalAxisLength = Extent.Z;
		}
		int32 NumberOfCells = FMath::CeilToInt32(PrincipalAxisLength / CellSize);
		check(NumberOfCells > 0);

		CellSize = PrincipalAxisLength / (FReal)NumberOfCells;
		FReal CellSizeInv = 1 / CellSize;

		int32 NumberOfCellsX = FMath::CeilToInt32(Extent.X * CellSizeInv) + 1;
		int32 NumberOfCellsY = FMath::CeilToInt32(Extent.Y * CellSizeInv) + 1;
		int32 NumberOfCellsZ = FMath::CeilToInt32(Extent.Z * CellSizeInv) + 1;

		// Create a Hash Table, but only store the buckets with constraint(s) as a map
		// HashTableMap<BucketIdx, ConstraintIdx>
		int32 NumberOfCellsXY = NumberOfCellsX * NumberOfCellsY;
		int32 NumberOfCellsXYZ = NumberOfCellsX * NumberOfCellsY * NumberOfCellsZ;
		for (int32 IdxConstraint = 0; IdxConstraint < ParticleArray.Num(); ++IdxConstraint)
		{
			FVector Location = ParticleArray[IdxConstraint] - BoundingBox.Min + FVector(0.5f * CellSize);
			int32 HashTableIdx = (int32)(Location.X * CellSizeInv) +
								 (int32)(Location.Y * CellSizeInv) * NumberOfCellsX +
								 (int32)(Location.Z * CellSizeInv) * NumberOfCellsXY;
			if (ensure(HashTableIdx < NumberOfCellsXYZ))
			{
				HashTableMap.Add(HashTableIdx, IdxConstraint);
			}
		}
	}

	void ComputeHashTable(const TArray<FBreakingData>& BreakingsArray,
						  const FBox& BoundingBox, TMultiMap<int32, int32>& HashTableMap, const FReal SpatialHashRadius)
	{
		FReal CellSize = 2 * SpatialHashRadius;
		check(CellSize > 0);

		// Compute number of cells along the principal axis
		FVector Extent = 2 * BoundingBox.GetExtent();
		FReal PrincipalAxisLength;
		if (Extent.X > Extent.Y && Extent.X > Extent.Z)
		{
			PrincipalAxisLength = Extent.X;
		}
		else if (Extent.Y > Extent.Z)
		{
			PrincipalAxisLength = Extent.Y;
		}
		else
		{
			PrincipalAxisLength = Extent.Z;
		}
		int32 NumberOfCells = FMath::CeilToInt32(PrincipalAxisLength / CellSize);
		check(NumberOfCells > 0);

		CellSize = PrincipalAxisLength / (FReal)NumberOfCells;
		FReal CellSizeInv = 1 / CellSize;

		int32 NumberOfCellsX = FMath::CeilToInt32(Extent.X * CellSizeInv) + 1;
		int32 NumberOfCellsY = FMath::CeilToInt32(Extent.Y * CellSizeInv) + 1;
		int32 NumberOfCellsZ = FMath::CeilToInt32(Extent.Z * CellSizeInv) + 1;

		// Create a Hash Table, but only store the buckets with constraint(s) as a map
		// HashTableMap<BucketIdx, ConstraintIdx>
		int32 NumberOfCellsXY = NumberOfCellsX * NumberOfCellsY;
		int32 NumberOfCellsXYZ = NumberOfCellsX * NumberOfCellsY * NumberOfCellsZ;
		for (int32 IdxBreaking = 0; IdxBreaking < BreakingsArray.Num(); ++IdxBreaking)
		{
			FVector Location = (FVector)BreakingsArray[IdxBreaking].Location - BoundingBox.Min + FVector(0.5f * CellSize);
			int32 HashTableIdx = (int32)(Location.X * CellSizeInv) +
								 (int32)(Location.Y * CellSizeInv) * NumberOfCellsX +
								 (int32)(Location.Z * CellSizeInv) * NumberOfCellsXY;
			if (ensure(HashTableIdx < NumberOfCellsXYZ))
			{
				HashTableMap.Add(HashTableIdx, IdxBreaking);
			}
		}
	}

	void ComputeHashTable(const TArray<FBreakingDataExt>& BreakingsArray,
						  const FBox& BoundingBox, TMultiMap<int32, int32>& HashTableMap, const FReal SpatialHashRadius)
	{
		FReal CellSize = 2 * SpatialHashRadius;
		check(CellSize > 0);

		// Compute number of cells along the principal axis
		FVector Extent = 2 * BoundingBox.GetExtent();
		FReal PrincipalAxisLength;
		if (Extent.X > Extent.Y && Extent.X > Extent.Z)
		{
			PrincipalAxisLength = Extent.X;
		}
		else if (Extent.Y > Extent.Z)
		{
			PrincipalAxisLength = Extent.Y;
		}
		else
		{
			PrincipalAxisLength = Extent.Z;
		}
		int32 NumberOfCells = FMath::CeilToInt32(PrincipalAxisLength / CellSize);
		check(NumberOfCells > 0);

		CellSize = PrincipalAxisLength / (FReal)NumberOfCells;
		FReal CellSizeInv = 1 / CellSize;

		int32 NumberOfCellsX = FMath::CeilToInt32(Extent.X * CellSizeInv) + 1;
		int32 NumberOfCellsY = FMath::CeilToInt32(Extent.Y * CellSizeInv) + 1;
		int32 NumberOfCellsZ = FMath::CeilToInt32(Extent.Z * CellSizeInv) + 1;

		// Create a Hash Table, but only store the buckets with constraint(s) as a map
		// HashTableMap<BucketIdx, ConstraintIdx>
		int32 NumberOfCellsXY = NumberOfCellsX * NumberOfCellsY;
		int32 NumberOfCellsXYZ = NumberOfCellsX * NumberOfCellsY * NumberOfCellsZ;
		for (int32 IdxBreaking = 0; IdxBreaking < BreakingsArray.Num(); ++IdxBreaking)
		{
			FVector Location = (FVector)BreakingsArray[IdxBreaking].Location - BoundingBox.Min + FVector(0.5f * CellSize);
			int32 HashTableIdx = (int32)(Location.X * CellSizeInv) +
				(int32)(Location.Y * CellSizeInv) * NumberOfCellsX +
				(int32)(Location.Z * CellSizeInv) * NumberOfCellsXY;
			if (ensure(HashTableIdx < NumberOfCellsXYZ))
			{
				HashTableMap.Add(HashTableIdx, IdxBreaking);
			}
		}
	}

}

