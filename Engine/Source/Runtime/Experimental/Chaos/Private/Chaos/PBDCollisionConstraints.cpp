// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/PBDCollisionConstraints.h"

#include "Chaos/CastingUtilities.h"
#include "Chaos/ChaosPerfTest.h"
#include "Chaos/ContactModification.h"
#include "Chaos/CCDModification.h"
#include "Chaos/CollisionResolution.h"
#include "Chaos/Collision/CollisionPruning.h"
#include "Chaos/Collision/PBDCollisionContainerSolver.h"
#include "Chaos/Collision/PBDCollisionContainerSolverJacobi.h"
#include "Chaos/Collision/PBDCollisionContainerSolverSimd.h"
#include "Chaos/Defines.h"
#include "Chaos/Evolution/SolverBodyContainer.h"
#include "Chaos/Evolution/ABTestingConstraintContainerSolver.h"
#include "Chaos/GeometryQueries.h"
#include "Chaos/Island/IslandManager.h"
#include "Chaos/PBDCollisionConstraintsContact.h"
#include "Chaos/PBDRigidsSOAs.h"
#include "Chaos/PhysicsMaterialUtilities.h"
#include "Chaos/SpatialAccelerationCollection.h"
#include "ChaosLog.h"
#include "ChaosStats.h"
#include "ProfilingDebugging/ScopedTimers.h"

//PRAGMA_DISABLE_OPTIMIZATION

namespace Chaos
{
	namespace CVars
	{
		extern bool bChaos_PBDCollisionSolver_UseJacobiPairSolver2;
		
		extern bool bChaosSolverPersistentGraph;
	}

	int32 CollisionParticlesBVHDepth = 4;
	FAutoConsoleVariableRef CVarCollisionParticlesBVHDepth(TEXT("p.CollisionParticlesBVHDepth"), CollisionParticlesBVHDepth, TEXT("The maximum depth for collision particles bvh"));

	int32 ConstraintBPBVHDepth = 2;
	FAutoConsoleVariableRef CVarConstraintBPBVHDepth(TEXT("p.ConstraintBPBVHDepth"), ConstraintBPBVHDepth, TEXT("The maximum depth for constraint bvh"));

	int32 BPTreeOfGrids = 1;
	FAutoConsoleVariableRef CVarBPTreeOfGrids(TEXT("p.BPTreeOfGrids"), BPTreeOfGrids, TEXT("Whether to use a seperate tree of grids for bp"));

	FRealSingle CollisionFrictionOverride = -1.0f;
	FAutoConsoleVariableRef CVarCollisionFrictionOverride(TEXT("p.CollisionFriction"), CollisionFrictionOverride, TEXT("Collision friction for all contacts if >= 0"));

	FRealSingle CollisionRestitutionOverride = -1.0f;
	FAutoConsoleVariableRef CVarCollisionRestitutionOverride(TEXT("p.CollisionRestitution"), CollisionRestitutionOverride, TEXT("Collision restitution for all contacts if >= 0"));
	
	FRealSingle CollisionAngularFrictionOverride = -1.0f;
	FAutoConsoleVariableRef CVarCollisionAngularFrictionOverride(TEXT("p.CollisionAngularFriction"), CollisionAngularFrictionOverride, TEXT("Collision angular friction for all contacts if >= 0"));

	FRealSingle CollisionBaseFrictionImpulseOverride = -1.0f;
	FAutoConsoleVariableRef CVarCollisionBaseFrictionImpulseOverride(TEXT("p.CollisionBaseFrictionImpulse"), CollisionBaseFrictionImpulseOverride, TEXT("Collision base friction position impulse for all contacts if >= 0"));

	CHAOS_API int32 EnableCollisions = 1;
	FAutoConsoleVariableRef CVarEnableCollisions(TEXT("p.EnableCollisions"), EnableCollisions, TEXT("Enable/Disable collisions on the Chaos solver."));
	
	FRealSingle DefaultCollisionFriction = 0;
	FAutoConsoleVariableRef CVarDefaultCollisionFriction(TEXT("p.DefaultCollisionFriction"), DefaultCollisionFriction, TEXT("Collision friction default value if no materials are found."));

	FRealSingle DefaultCollisionRestitution = 0;
	FAutoConsoleVariableRef CVarDefaultCollisionRestitution(TEXT("p.DefaultCollisionRestitution"), DefaultCollisionRestitution, TEXT("Collision restitution default value if no materials are found."));

	FRealSingle CollisionRestitutionThresholdOverride = -1.0f;
	FAutoConsoleVariableRef CVarDefaultCollisionRestitutionThreshold(TEXT("p.CollisionRestitutionThreshold"), CollisionRestitutionThresholdOverride, TEXT("Collision restitution threshold override if >= 0 (units of acceleration)"));

	int32 CollisionCanAlwaysDisableContacts = 0;
	FAutoConsoleVariableRef CVarCollisionCanAlwaysDisableContacts(TEXT("p.CollisionCanAlwaysDisableContacts"), CollisionCanAlwaysDisableContacts, TEXT("Collision culling will always be able to permanently disable contacts"));

	int32 CollisionCanNeverDisableContacts = 0;
	FAutoConsoleVariableRef CVarCollisionCanNeverDisableContacts(TEXT("p.CollisionCanNeverDisableContacts"), CollisionCanNeverDisableContacts, TEXT("Collision culling will never be able to permanently disable contacts"));

	bool CollisionsAllowParticleTracking = true;
	FAutoConsoleVariableRef CVarCollisionsAllowParticleTracking(TEXT("p.Chaos.Collision.AllowParticleTracking"), CollisionsAllowParticleTracking, TEXT("Allow particles to track their collisions constraints when their DoBufferCollisions flag is enable [def:true]"));

	//	Which edge pruning features to enable (for particle swith EdgeSmoothing enabled)
	bool bCollisionsEnableEdgeCollisionPruning = true;
	bool bCollisionsEnableMeshCollisionPruning = true;
	bool bCollisionsEnableSubSurfaceCollisionPruning = false;
	FAutoConsoleVariableRef CVarCollisionsEnableEdgeCollisionPruning(TEXT("p.Chaos.Collision.EnableEdgeCollisionPruning"), bCollisionsEnableEdgeCollisionPruning, TEXT(""));
	FAutoConsoleVariableRef CVarCollisionsEnableMeshCollisionPruning(TEXT("p.Chaos.Collision.EnableMeshCollisionPruning"), bCollisionsEnableMeshCollisionPruning, TEXT(""));
	FAutoConsoleVariableRef CVarCollisionsEnableSubSurfaceCollisionPruning(TEXT("p.Chaos.Collision.EnableSubSurfaceCollisionPruning"), bCollisionsEnableSubSurfaceCollisionPruning, TEXT(""));

	bool DebugDrawProbeDetection = false;
	FAutoConsoleVariableRef CVarDebugDrawProbeDetection(TEXT("p.Chaos.Collision.DebugDrawProbeDetection"), DebugDrawProbeDetection, TEXT("Draw probe constraint detection."));

#if CHAOS_DEBUG_DRAW
	namespace CVars
	{
		extern DebugDraw::FChaosDebugDrawSettings ChaosSolverDebugDebugDrawSettings;
	}
#endif
	
	DECLARE_CYCLE_STAT(TEXT("Collisions::Reset"), STAT_Collisions_Reset, STATGROUP_ChaosCollision);
	DECLARE_CYCLE_STAT(TEXT("Collisions::BeginDetect"), STAT_Collisions_BeginDetect, STATGROUP_ChaosCollision);
	DECLARE_CYCLE_STAT(TEXT("Collisions::EndDetect"), STAT_Collisions_EndDetect, STATGROUP_ChaosCollision);
	DECLARE_CYCLE_STAT(TEXT("Collisions::Sort"), STAT_Collisions_Sort, STATGROUP_ChaosCollision);
	DECLARE_CYCLE_STAT(TEXT("Collisions::DetectProbeCollisions"), STAT_Collisions_DetectProbeCollisions, STATGROUP_ChaosCollision);

#if CHAOS_ABTEST_CONSTRAINTSOLVER_ENABLED
	namespace CVars
	{
		bool bCollisionsEnableSolverABTest = false;
		FAutoConsoleVariableRef CVarCollisionsEnableSolverABTest(TEXT("p.Chaos.Collision.ABTestSolver"), bCollisionsEnableSolverABTest, TEXT(""));
	}

	// An AB Testing collision solver for use while developing the Simd version
	using FABTestingCollisionContainerSolver = Private::TABTestingConstraintContainerSolver<FPBDCollisionContainerSolver, Private::FPBDCollisionContainerSolverSimd>;

	// Simd AB testing callback
	void ABTestSimdCollisionSolver(
		FABTestingCollisionContainerSolver::ESolverPhase Phase,
		const FPBDCollisionContainerSolver& SolverA,
		const Private::FPBDCollisionContainerSolverSimd& SolverB,
		const FSolverBodyContainer& SolverBodyContainerA,
		const FSolverBodyContainer& SolverBodyContainerB)
	{
		for (int32 ConstraintIndex = 0; ConstraintIndex < SolverA.GetNumConstraints(); ++ConstraintIndex)
		{
			const Private::FPBDCollisionSolver& ConstraintSolverA = SolverA.GetConstraintSolver(ConstraintIndex);
			const FSolverBody& Body0 = ConstraintSolverA.SolverBody0().SolverBody();
			const FSolverBody& Body1 = ConstraintSolverA.SolverBody1().SolverBody();

			const Private::FPBDCollisionContainerSolverSimd::FConstraintSolverId ConstraintSolverBId = SolverB.GetConstraintSolverId(ConstraintIndex);
			const Private::TPBDCollisionSolverSimd<4>& ConstraintSolverB = SolverB.GetConstraintSolver(ConstraintSolverBId.SolverIndex);
			const int32 LaneIndexB = ConstraintSolverBId.LaneIndex;

			if (ConstraintSolverA.NumManifoldPoints() != ConstraintSolverB.NumManifoldPoints().GetValue(LaneIndexB))
			{
				UE_LOGF(LogChaos, Warning, "ManifoldPoint count mismatch");
				return;
			}

			for (int32 ManifoldPointIndex = 0; ManifoldPointIndex < ConstraintSolverA.NumManifoldPoints(); ++ManifoldPointIndex)
			{
				const Private::FPBDCollisionSolverManifoldPoint& ManifoldPointSolverA = ConstraintSolverA.GetManifoldPoint(ManifoldPointIndex);
				const Private::TPBDCollisionSolverManifoldPointsSimd<4>& ManifoldPointSolverB = ConstraintSolverB.GetManifoldPoint(ManifoldPointIndex, SolverB.GetManifoldPointBuffer());
				bool bIsError = false;

				if (ManifoldPointSolverA.ContactNormal != ManifoldPointSolverB.SimdContactNormal.GetValue(LaneIndexB))
				{
					UE_LOGF(LogChaos, Warning, "ContactNormal mismatch");
					bIsError = true;
				}
				if (ManifoldPointSolverA.ContactTangentU != ManifoldPointSolverB.SimdContactTangentU.GetValue(LaneIndexB))
				{
					UE_LOGF(LogChaos, Warning, "ContactTangentU mismatch");
					bIsError = true;
				}
				if (ManifoldPointSolverA.ContactTangentV != ManifoldPointSolverB.SimdContactTangentV.GetValue(LaneIndexB))
				{
					UE_LOGF(LogChaos, Warning, "ContactTangentV mismatch");
					bIsError = true;
				}
				if (ManifoldPointSolverA.RelativeContactPoints[0] != ManifoldPointSolverB.SimdRelativeContactPoint0.GetValue(LaneIndexB))
				{
					UE_LOGF(LogChaos, Warning, "RelativeContactPoints mismatch");
					bIsError = true;
				}
				if (ManifoldPointSolverA.RelativeContactPoints[1] != ManifoldPointSolverB.SimdRelativeContactPoint1.GetValue(LaneIndexB))
				{
					UE_LOGF(LogChaos, Warning, "RelativeContactPoints mismatch");
					bIsError = true;
				}
				if (ManifoldPointSolverA.ContactMassNormal != ManifoldPointSolverB.SimdContactMassNormal.GetValue(LaneIndexB))
				{
					UE_LOGF(LogChaos, Warning, "ContactMassNormal mismatch");
					bIsError = true;
				}
				if (ManifoldPointSolverA.ContactMassTangentU != ManifoldPointSolverB.SimdContactMassTangentU.GetValue(LaneIndexB))
				{
					UE_LOGF(LogChaos, Warning, "ContactMassTangentU mismatch");
					bIsError = true;
				}
				if (ManifoldPointSolverA.ContactMassTangentV != ManifoldPointSolverB.SimdContactMassTangentV.GetValue(LaneIndexB))
				{
					UE_LOGF(LogChaos, Warning, "ContactMassTangentV mismatch");
					bIsError = true;
				}
				if (Body0.IsDynamic())
				{
					if (ManifoldPointSolverA.ContactTangentUAngular0 != ManifoldPointSolverB.SimdContactTangentUAngular0.GetValue(LaneIndexB))
					{
						UE_LOGF(LogChaos, Warning, "ContactTangentUAngular0 mismatch");
						bIsError = true;
					}
					if (ManifoldPointSolverA.ContactTangentVAngular0 != ManifoldPointSolverB.SimdContactTangentVAngular0.GetValue(LaneIndexB))
					{
						UE_LOGF(LogChaos, Warning, "ContactTangentVAngular0 mismatch");
						bIsError = true;
					}
				}
				if (Body1.IsDynamic())
				{
					if (ManifoldPointSolverA.ContactTangentUAngular1 != ManifoldPointSolverB.SimdContactTangentUAngular1.GetValue(LaneIndexB))
					{
						UE_LOGF(LogChaos, Warning, "ContactTangentUAngular1 mismatch");
						bIsError = true;
					}
					if (ManifoldPointSolverA.ContactTangentVAngular1 != ManifoldPointSolverB.SimdContactTangentVAngular1.GetValue(LaneIndexB))
					{
						UE_LOGF(LogChaos, Warning, "ContactTangentVAngular1 mismatch");
						bIsError = true;
					}
				}

				if (Phase == FABTestingCollisionContainerSolver::ESolverPhase::PostApplyPositionConstraints)
				{
					if (ManifoldPointSolverA.NetPushOutNormal != ManifoldPointSolverB.SimdNetPushOutNormal.GetValue(LaneIndexB))
					{
						UE_LOGF(LogChaos, Warning, "NetPushOutNormal mismatch");
						bIsError = true;
					}
					if (ManifoldPointSolverA.NetPushOutTangentU != ManifoldPointSolverB.SimdNetPushOutTangentU.GetValue(LaneIndexB))
					{
						UE_LOGF(LogChaos, Warning, "NetPushOutTangentU mismatch");
						bIsError = true;
					}
					if (ManifoldPointSolverA.NetPushOutTangentV != ManifoldPointSolverB.SimdNetPushOutTangentV.GetValue(LaneIndexB))
					{
						UE_LOGF(LogChaos, Warning, "NetPushOutTangentV mismatch");
						bIsError = true;
					}
				}

				if (Phase == FABTestingCollisionContainerSolver::ESolverPhase::PostApplyVelocityConstraints)
				{
					if (ManifoldPointSolverA.NetImpulseNormal != ManifoldPointSolverB.SimdNetImpulseNormal.GetValue(LaneIndexB))
					{
						UE_LOGF(LogChaos, Warning, "NetImpulseNormal mismatch");
						bIsError = true;
					}
					if (ManifoldPointSolverA.NetImpulseTangentU != ManifoldPointSolverB.SimdNetImpulseTangentU.GetValue(LaneIndexB))
					{
						UE_LOGF(LogChaos, Warning, "NetImpulseTangentU mismatch");
						bIsError = true;
					}
					if (ManifoldPointSolverA.NetImpulseTangentV != ManifoldPointSolverB.SimdNetImpulseTangentV.GetValue(LaneIndexB))
					{
						UE_LOGF(LogChaos, Warning, "NetImpulseTangentV mismatch");
						bIsError = true;
					}
				}

				if (bIsError)
				{
					UE_LOGF(LogChaos, Warning, "SimdCollisionSolver mismatch");
				}
			}
		}

		if (SolverBodyContainerA.Num() != SolverBodyContainerB.Num())
		{
			UE_LOGF(LogChaos, Warning, "Body count mismatch");
			return;
		}

		for (int BodyIndex = 0; BodyIndex < SolverBodyContainerA.Num(); ++BodyIndex)
		{
			const FSolverBody& BodyA = SolverBodyContainerA.GetSolverBody(BodyIndex);
			const FSolverBody& BodyB = SolverBodyContainerB.GetSolverBody(BodyIndex);
			bool bIsError = false;

			if (BodyA.P() != BodyB.P())
			{
				UE_LOGF(LogChaos, Warning, "Position mismatch");
				bIsError = true;
			}

			if (BodyA.Q() != BodyB.Q())
			{
				UE_LOGF(LogChaos, Warning, "Rotation mismatch");
				bIsError = true;
			}

			if (BodyA.DP() != BodyB.DP())
			{
				UE_LOGF(LogChaos, Warning, "PositionDelta mismatch");
				bIsError = true;
			}

			if (BodyA.DQ() != BodyB.DQ())
			{
				UE_LOGF(LogChaos, Warning, "RotationDelta mismatch");
				bIsError = true;
			}

			if (BodyA.V() != BodyB.V())
			{
				UE_LOGF(LogChaos, Warning, "Velocity mismatch");
				bIsError = true;
			}

			if (BodyA.W() != BodyB.W())
			{
				UE_LOGF(LogChaos, Warning, "AngVel mismatch");
				bIsError = true;
			}

			if (BodyA.InvM() != BodyB.InvM())
			{
				UE_LOGF(LogChaos, Warning, "InvM mismatch");
				bIsError = true;
			}

			if (BodyA.InvI() != BodyB.InvI())
			{
				UE_LOGF(LogChaos, Warning, "InvI mismatch");
				bIsError = true;
			}

			if (bIsError)
			{
				UE_LOGF(LogChaos, Warning, "SimdCollisionSolver mismatch");
			}
		}
	}
#endif

	//
	// Collision Constraint Container
	//

	FPBDCollisionConstraints::FPBDCollisionConstraints(
		const FPBDRigidsSOAs& InParticles,
		TArrayCollectionArray<bool>& Collided,
		const TArrayCollectionArray<TSerializablePtr<FChaosPhysicsMaterial>>& InPhysicsMaterials,
		const TArrayCollectionArray<TUniquePtr<FChaosPhysicsMaterial>>& InPerParticlePhysicsMaterials,
		const THandleArray<FChaosPhysicsMaterial>* const InSimMaterials,
		const int32 NumCollisionsPerBlock,
		const FReal InRestitutionThreshold)
		: FPBDConstraintContainer(FConstraintContainerHandle::StaticType())
		, Particles(InParticles)
		, ConstraintAllocator(NumCollisionsPerBlock)
		, NumActivePointConstraints(0)
		, MCollided(Collided)
		, MPhysicsMaterials(InPhysicsMaterials)
		, MPerParticlePhysicsMaterials(InPerParticlePhysicsMaterials)
		, SimMaterials(InSimMaterials)
		, RestitutionThreshold(InRestitutionThreshold)	// @todo(chaos): expose as property
		, bEnableCollisions(true)
		, bEnableRestitution(true)
		, bHandlesEnabled(true)
		, bEnableEdgePruning(true)
		, bIsDeterministic(false)
		, bCanDisableContacts(true)
		, CollisionSolverType(Private::ECollisionSolverType::GaussSeidel)
		, GravityDirection(FVec3(0,0,-1))
		, GravitySize(980)
		, SolverSettings()
	{
		// Unfortunately, but the collision it creates need to know what container they belong to,
		// but otherwise the allocator doesn't really need to know about the container...
		ConstraintAllocator.SetCollisionContainer(this);
	}

	FPBDCollisionConstraints::~FPBDCollisionConstraints()
	{
	}

	TUniquePtr<FConstraintContainerSolver> FPBDCollisionConstraints::CreateSceneSolver(const int32 Priority)
	{
		// RBAN always uses Gauss Seidel solver for now
		return MakeUnique<FPBDCollisionContainerSolver>(*this, Priority);
	}

	TUniquePtr<FConstraintContainerSolver> FPBDCollisionConstraints::CreateGroupSolver(const int32 Priority)
	{
		switch (CollisionSolverType)
		{
		case Private::ECollisionSolverType::GaussSeidel:
			return MakeUnique<FPBDCollisionContainerSolver>(*this, Priority);

		case Private::ECollisionSolverType::GaussSeidelSimd:
#if CHAOS_ABTEST_CONSTRAINTSOLVER_ENABLED
			if (CVars::bCollisionsEnableSolverABTest)
			{
				// Create an AB testing collision solver for simd testing
				return MakeUnique<FABTestingCollisionContainerSolver>(
					MakeUnique<FPBDCollisionContainerSolver>(*this, Priority),
					MakeUnique<Private::FPBDCollisionContainerSolverSimd>(*this, Priority),
					Priority,
					&ABTestSimdCollisionSolver);
			}
#endif
			return MakeUnique<Private::FPBDCollisionContainerSolverSimd>(*this, Priority);

		case Private::ECollisionSolverType::PartialJacobi:
			return MakeUnique<Private::FPBDCollisionContainerSolverJacobi>(*this, Priority);
		}

		check(false);
		return nullptr;
	}

	void FPBDCollisionConstraints::SetIsDeterministic(const bool bInIsDeterministic)
	{
		bIsDeterministic = bInIsDeterministic;
		ConstraintAllocator.SetIsDeterministic(bInIsDeterministic);
	}

	bool FPBDCollisionConstraints::GetIsDeterministic() const
	{
		return bIsDeterministic;
	}

	void FPBDCollisionConstraints::DisableHandles()
	{
		check(NumConstraints() == 0);
		bHandlesEnabled = false;
	}

	FPBDCollisionConstraints::FHandles FPBDCollisionConstraints::GetConstraintHandles() const
	{
		return ConstraintAllocator.GetConstraints();
	}

	FPBDCollisionConstraints::FConstHandles FPBDCollisionConstraints::GetConstConstraintHandles() const
	{
		return ConstraintAllocator.GetConstConstraints();
	}

	void UpdateSoftCollisionSettings(const FChaosPhysicsMaterial* PhysicsMaterial, const FGeometryParticleHandle* Particle, FReal& InOutThickess)
	{
		if ((PhysicsMaterial->SoftCollisionMode != EChaosPhysicsMaterialSoftCollisionMode::None) && (PhysicsMaterial->SoftCollisionThickness > 0))
		{
			FReal MaterialThickness = FReal(0);
			if (PhysicsMaterial->SoftCollisionMode == EChaosPhysicsMaterialSoftCollisionMode::AbsoluteThickness)
			{
				MaterialThickness = PhysicsMaterial->SoftCollisionThickness;
			}
			else if ((Particle != nullptr) && Particle->HasBounds())
			{
				MaterialThickness = FMath::Clamp(PhysicsMaterial->SoftCollisionThickness, 0.0f, 0.5f) * Particle->LocalBounds().Extents().GetAbsMin();
			}

			InOutThickess = FMath::Max(InOutThickess, MaterialThickness);
		}
	}

	void FPBDCollisionConstraints::UpdateConstraintMaterialProperties(FPBDCollisionConstraint& Constraint)
	{
		// We only support one material shared by all manifold points for now, even when our 
		// 4 manifold points are on different triangles of a mesh for example
		int32 ShapeFaceIndex = INDEX_NONE;
		if (Constraint.NumManifoldPoints() > 0)
		{
			ShapeFaceIndex = Constraint.GetManifoldPoint(0).ContactPoint.FaceIndex;
		}

		// This is a bit dodgy - we pass the FaceIndex to both material requests, knowing that at most one of the shapes will use it
		const FChaosPhysicsMaterial* PhysicsMaterial0 = Private::GetPhysicsMaterial(Constraint.Particle[0], Constraint.GetShape0(), ShapeFaceIndex, &MPhysicsMaterials, &MPerParticlePhysicsMaterials, SimMaterials);
		const FChaosPhysicsMaterial* PhysicsMaterial1 = Private::GetPhysicsMaterial(Constraint.Particle[1], Constraint.GetShape1(), ShapeFaceIndex, &MPhysicsMaterials, &MPerParticlePhysicsMaterials, SimMaterials);

		FReal MaterialRestitution = 0;
		FReal MaterialRestitutionThreshold = 0;
		FReal MaterialStaticFriction = 0;
		FReal MaterialDynamicFriction = 0;
		FReal MaterialSoftThickness = 0;
		FReal MaterialSoftDivisor = 0;
		FReal MaterialBaseFrictionImpulse = 0;

		if (PhysicsMaterial0 && PhysicsMaterial1)
		{
			const FChaosPhysicsMaterial::ECombineMode RestitutionCombineMode = FChaosPhysicsMaterial::ChooseCombineMode(PhysicsMaterial0->RestitutionCombineMode,PhysicsMaterial1->RestitutionCombineMode);
			MaterialRestitution = FChaosPhysicsMaterial::CombineHelper(PhysicsMaterial0->Restitution, PhysicsMaterial1->Restitution, RestitutionCombineMode);

			const FChaosPhysicsMaterial::ECombineMode FrictionCombineMode = FChaosPhysicsMaterial::ChooseCombineMode(PhysicsMaterial0->FrictionCombineMode,PhysicsMaterial1->FrictionCombineMode);
			MaterialDynamicFriction = FChaosPhysicsMaterial::CombineHelper(PhysicsMaterial0->Friction,PhysicsMaterial1->Friction, FrictionCombineMode);
			const FReal StaticFriction0 = FMath::Max(PhysicsMaterial0->Friction, PhysicsMaterial0->StaticFriction);
			const FReal StaticFriction1 = FMath::Max(PhysicsMaterial1->Friction, PhysicsMaterial1->StaticFriction);
			MaterialStaticFriction = FChaosPhysicsMaterial::CombineHelper(StaticFriction0, StaticFriction1, FrictionCombineMode);

			// @todo(chaos): could do with a nicer way to deal with collisions between two soft objects with different softness settings
			UpdateSoftCollisionSettings(PhysicsMaterial0, Constraint.GetParticle0(), MaterialSoftThickness);
			UpdateSoftCollisionSettings(PhysicsMaterial1, Constraint.GetParticle1(), MaterialSoftThickness);

			// Combine base friction impulse
			MaterialBaseFrictionImpulse = FChaosPhysicsMaterial::CombineHelper(PhysicsMaterial0->BaseFrictionImpulse, PhysicsMaterial1->BaseFrictionImpulse, FrictionCombineMode);
		}
		else if (PhysicsMaterial0)
		{
			const FReal StaticFriction0 = FMath::Max(PhysicsMaterial0->Friction, PhysicsMaterial0->StaticFriction);
			MaterialRestitution = PhysicsMaterial0->Restitution;
			MaterialDynamicFriction = PhysicsMaterial0->Friction;
			MaterialStaticFriction = StaticFriction0;
			MaterialBaseFrictionImpulse = PhysicsMaterial0->BaseFrictionImpulse;
			UpdateSoftCollisionSettings(PhysicsMaterial0, Constraint.GetParticle0(), MaterialSoftThickness);
		}
		else if (PhysicsMaterial1)
		{
			const FReal StaticFriction1 = FMath::Max(PhysicsMaterial1->Friction, PhysicsMaterial1->StaticFriction);
			MaterialRestitution = PhysicsMaterial1->Restitution;
			MaterialDynamicFriction = PhysicsMaterial1->Friction;
			MaterialStaticFriction = StaticFriction1;
			MaterialBaseFrictionImpulse = PhysicsMaterial1->BaseFrictionImpulse;
			UpdateSoftCollisionSettings(PhysicsMaterial1, Constraint.GetParticle1(), MaterialSoftThickness);
		}
		else
		{
			MaterialDynamicFriction = DefaultCollisionFriction;
			MaterialStaticFriction = DefaultCollisionFriction;
			MaterialRestitution = DefaultCollisionRestitution;
		}

		MaterialRestitutionThreshold = RestitutionThreshold;

		// Overrides for testing
		if (CollisionFrictionOverride >= 0)
		{
			MaterialDynamicFriction = CollisionFrictionOverride;
			MaterialStaticFriction = CollisionFrictionOverride;
		}
		if (CollisionRestitutionOverride >= 0)
		{
			MaterialRestitution = CollisionRestitutionOverride;
		}
		if (CollisionRestitutionThresholdOverride >= 0.0f)
		{
			MaterialRestitutionThreshold = CollisionRestitutionThresholdOverride;
		}
		if (CollisionAngularFrictionOverride >= 0)
		{
			MaterialStaticFriction = CollisionAngularFrictionOverride;
		}
		if (CollisionBaseFrictionImpulseOverride >= 0)
		{
			MaterialBaseFrictionImpulse = CollisionBaseFrictionImpulseOverride;
		}
		if (!bEnableRestitution)
		{
			MaterialRestitution = 0.0f;
		}
		
		Constraint.Material.Restitution = FRealSingle(MaterialRestitution);
		Constraint.Material.RestitutionThreshold = FRealSingle(MaterialRestitutionThreshold);
		Constraint.Material.StaticFriction = FRealSingle(MaterialStaticFriction);
		Constraint.Material.DynamicFriction = FRealSingle(MaterialDynamicFriction);
		Constraint.Material.SoftSeparation = FRealSingle(-MaterialSoftThickness);	// Negate: convert from penetration to separation
		Constraint.Material.InvMassScale0 = 1;
		Constraint.Material.InvMassScale1 = 1;
		Constraint.Material.InvInertiaScale0 = 1;
		Constraint.Material.InvInertiaScale1 = 1;
		Constraint.Material.BaseFrictionImpulse = FRealSingle(MaterialBaseFrictionImpulse);
	}

	void FPBDCollisionConstraints::BeginFrame()
	{
		ConstraintAllocator.BeginFrame();
	}

	void FPBDCollisionConstraints::Reset()
	{
		SCOPE_CYCLE_COUNTER(STAT_Collisions_Reset);

		ConstraintAllocator.Reset();
	}

	void FPBDCollisionConstraints::BeginDetectCollisions()
	{
		SCOPE_CYCLE_COUNTER(STAT_Collisions_BeginDetect);

		ConstraintAllocator.BeginDetectCollisions();
	}

	void FPBDCollisionConstraints::EndDetectCollisions()
	{
		SCOPE_CYCLE_COUNTER(STAT_Collisions_EndDetect);

		// Prune the unused contacts
		ConstraintAllocator.EndDetectCollisions();

		// Disable any edge collisions that are hidden by face collisions
		// (for bodies that have the EdgePruning option enabled)
		PruneEdgeCollisions();
	}

	void FPBDCollisionConstraints::DetectProbeCollisions(FReal Dt)
	{
		SCOPE_CYCLE_COUNTER(STAT_Collisions_DetectProbeCollisions);

		// Do a final narrow-phase update on probe constraints. This is done to avoid
		// false positive probe hit results, which may occur if the constraint was created
		// for a contact which no longer is occurring due to resolution of another constraint.
		for (FPBDCollisionConstraint* Contact : GetConstraints())
		{
			if ((Contact != nullptr) && Contact->IsProbe())
			{
				const FGeometryParticleHandle* Particle0 = Contact->GetParticle0();
				const FGeometryParticleHandle* Particle1 = Contact->GetParticle1();
				const FRigidTransform3& ShapeWorldTransform0 = Contact->GetShapeRelativeTransform0() * FParticleUtilities::GetActorWorldTransform(FConstGenericParticleHandle(Particle0));
				const FRigidTransform3& ShapeWorldTransform1 = Contact->GetShapeRelativeTransform1() * FParticleUtilities::GetActorWorldTransform(FConstGenericParticleHandle(Particle1));
				Contact->SetCullDistance(0.f);
				Contact->SetShapeWorldTransforms(ShapeWorldTransform0, ShapeWorldTransform1);
				Collisions::UpdateConstraint(*Contact, ShapeWorldTransform0, ShapeWorldTransform1, Dt);

#if CHAOS_DEBUG_DRAW
				if (DebugDrawProbeDetection)
				{
					DebugDraw::FChaosDebugDrawSettings Settings = CVars::ChaosSolverDebugDebugDrawSettings;
					Settings.DrawDuration = 1.f;
					DebugDraw::DrawCollidingShapes(FRigidTransform3(), *Contact, 1.f, 1.f, &Settings);
				}
#endif
			}
		}
	}

	void FPBDCollisionConstraints::ApplyCCDModifier(const TArray<ISimCallbackObject*>& CCDModifiers, FReal Dt)
	{
		FCCDModifierAccessor ModifierAccessor(Dt);
		for (ISimCallbackObject* ModifierCallback : CCDModifiers)
		{
			ModifierCallback->CCDModification_Internal(ModifierAccessor);
		}
	}

	void FPBDCollisionConstraints::ApplyCollisionModifier(const TArray<ISimCallbackObject*>& CollisionModifiers, FReal Dt)
	{
		if (GetConstraints().Num() > 0)
		{
			// NOTE: at this point, we will not have any nullptrs in the constraint handles
			TArrayView<FPBDCollisionConstraint* const> ConstraintHandles = GetConstraintHandles();
			FCollisionContactModifier Modifier(ConstraintHandles, Dt);

			for(ISimCallbackObject* ModifierCallback : CollisionModifiers)
			{
				FScopedTraceSolverCallback ScopedCallback(ModifierCallback);
				ModifierCallback->ContactModification_Internal(Modifier);
			}

			Modifier.UpdateConstraintManifolds();
		}
	}

	void FPBDCollisionConstraints::DisconnectConstraints(const TSet<FGeometryParticleHandle*>& ParticleHandles)
	{
		// Does nothing for collision constraints
	}

	void FPBDCollisionConstraints::RemoveConstraints(const TSet<FGeometryParticleHandle*>& ParticleHandles)
	{
		for (FGeometryParticleHandle* ParticleHandle : ParticleHandles)
		{
			ConstraintAllocator.RemoveParticle(ParticleHandle);
		}
	}

	void FPBDCollisionConstraints::AddConstraintsToGraph(Private::FPBDIslandManager& IslandManager)
	{
		// Debugging/diagnosing: if we have collisions disabled, remove all collisions from the graph and don't add any more
		if (!GetCollisionsEnabled())
		{
			IslandManager.RemoveContainerConstraints(GetContainerId());
			return;
		}

		// Remove expired collisions
		// @chaos(todo): if graph persistent is disabled we remove all collisions, but in a non-optimal way...
		TempCollisions.Reset();
		const bool bRemoveAllAwakeCollisions = !CVars::bChaosSolverPersistentGraph;
		IslandManager.VisitAwakeConstraints(GetContainerId(),
			[this, bRemoveAllAwakeCollisions](const Private::FPBDIslandConstraint* IslandConstraint)
			{
				FPBDCollisionConstraintHandle* CollisionHandle = IslandConstraint->GetConstraint()->AsUnsafe<FPBDCollisionConstraintHandle>();
				if (bRemoveAllAwakeCollisions || !CollisionHandle->IsEnabled() || CollisionHandle->IsProbe() || ConstraintAllocator.IsConstraintExpired(CollisionHandle->GetContact()))
				{
					TempCollisions.Add(CollisionHandle);
				}
			});

		// Remove expired constraints
		for (FPBDCollisionConstraintHandle* CollisionHandle : TempCollisions)
		{
			IslandManager.RemoveConstraint(CollisionHandle);
		}

		// Collect all the new constraints that need to be added to the graph
		TempCollisions.Reset();
		for (FPBDCollisionConstraintHandle* ConstraintHandle : ConstraintAllocator.GetConstraints())
		{
			FPBDCollisionConstraint& Constraint = ConstraintHandle->GetContact();
			if (!Constraint.IsInConstraintGraph() && Constraint.IsEnabled() && !Constraint.IsProbe())
			{
				TempCollisions.Add(ConstraintHandle);
			}
		}

		// Sort new constraints into a predictable order. This isn't strictly required if we don't need
		// deterministic behaviour, but without sorting we can get fairly different behaviour from run 
		// to run because the collisions detection order is effectively random on multicore machines.
		//
		// @todo(chaos): this is still not good enough for some types of determinism. Specifically if we 
		// create two set of objects in a different order but with the same physical positions and other 
		// state, they will behave differently which is undesirable. To fix this we need a sorting 
		// mechanism that does not rely on properties like Particle IDs, but it would be expensive.
		//
		// NOTE: If bIsDeterministic is true, we have already sorted the active constraints list 
		// (see EndDetectCollisions) so we don't need to do it again here
		if (!bIsDeterministic)
		{
			SCOPE_CYCLE_COUNTER(STAT_Collisions_Sort);

			TempCollisions.Sort(
				[](const FPBDCollisionConstraintHandle& L, const FPBDCollisionConstraintHandle& R)
				{
					return L.GetContact().GetCollisionSortKey() < R.GetContact().GetCollisionSortKey();
				});
		}

		// Add the new constraints to the graph
		for (FPBDCollisionConstraintHandle* ConstraintHandle : TempCollisions)
		{
			IslandManager.AddConstraint(GetContainerId(), ConstraintHandle, ConstraintHandle->GetConstrainedParticles());

			// Check for nodes whose transform was set explicitly, estimate the wake-up momentum with new constraints
			// Update the smooth velocity to an estimate of the motion which would have lead to the penetration detected.
			const auto UpdateSmoothVelocity = [](FGeometryParticleHandle* Particle, const FPBDCollisionConstraint& Contact)  -> void
				{
					if (FPBDRigidParticleHandle* Rigid = Particle->CastToRigidParticle())
					{
						// @todo(Chaos): Not great because
						// 1) The time step is hard-coded which might lead to unwanted side effect for frame rate changes
						// 2) VSmooth logic is spread out over various files and classes (IslandManager, PBDRigidsEvolutionGBF)
						// 3) Penetration velocity estimation is not what we would expect when calling AddToConstraintGraph
						const FReal FakeInvDt = 30.0; // hard-coded inverse time step
						const FReal Phi = Contact.GetPhi();
						const FVec3 Normal = Contact.CalculateWorldContactNormal();
						const FVec3 PenetrationVelocity = (Phi * Normal) * FakeInvDt;
						if (Rigid->VSmooth().SizeSquared() + UE_KINDA_SMALL_NUMBER < PenetrationVelocity.SizeSquared())
						{
							Rigid->SetVSmooth(PenetrationVelocity);
						}
					}
				};


			const FParticlePair& ParticlePair = ConstraintHandle->GetConstrainedParticles();
			const FPBDCollisionConstraint& Contact = ConstraintHandle->GetContact();

			if (IslandManager.WasParticleTransformSetExplicitly(ParticlePair[0]))
			{
				UpdateSmoothVelocity(ParticlePair[0], Contact);
			}
			if (IslandManager.WasParticleTransformSetExplicitly(ParticlePair[1]))
			{
				UpdateSmoothVelocity(ParticlePair[1], Contact);
			}
		}

		TempCollisions.Reset();
	}

	const FPBDCollisionConstraint& FPBDCollisionConstraints::GetConstraint(int32 Index) const
	{
		check(Index < NumConstraints());

		return *GetConstraints()[Index];
	}

	const FPBDCollisionConstraint& FPBDCollisionConstraints::GetConstConstraint(int32 Index) const
	{
		check(Index < NumConstraints());

		return *GetConstraints()[Index];
	}

	FPBDCollisionConstraint& FPBDCollisionConstraints::GetConstraint(int32 Index)
	{
		check(Index < NumConstraints());

		return *GetConstraints()[Index];
	}

	void FPBDCollisionConstraints::PruneEdgeCollisions()
	{
		if (bEnableEdgePruning)
		{
			for (auto& ParticleHandle : Particles.GetNonDisabledDynamicView())
			{
				if ((ParticleHandle.CollisionConstraintFlags() & (uint32)ECollisionConstraintFlags::CCF_SmoothEdgeCollisions) != 0)
				{
					if (bCollisionsEnableMeshCollisionPruning)
					{
						FParticleMeshCollisionPruner MeshPruner(ParticleHandle.Handle());
						MeshPruner.Prune();
					}

					if (bCollisionsEnableEdgeCollisionPruning)
					{
						FParticleEdgeCollisionPruner EdgePruner(ParticleHandle.Handle());
						EdgePruner.Prune();
					}

					if (bCollisionsEnableSubSurfaceCollisionPruning)
					{
						const FVec3 UpVector = ParticleHandle.GetR().GetAxisZ();
						FParticleSubSurfaceCollisionPruner SubSurfacePruner(ParticleHandle.Handle());
						SubSurfacePruner.Prune(UpVector);
					}
				}
			}
		}
	}

}
