// Copyright Epic Games, Inc. All Rights Reserved.

#define EXPLICIT_VECTOR4 1

#include "Math/Vector.isph"
#include "Math/Quat.isph"
#include "Math/Transform.isph"
#include "Chaos/Matrix33.isph"

struct TArrayND
{
	uniform FIntVector MCounts;
	uniform FReal * uniform MArray;
};

inline static FReal BoxPhi(const uniform FVector &BoxMin, const uniform FVector &BoxMax, const FVector& LocalPoint)
{
	const FVector MaxDists = LocalPoint - BoxMax;
	const FVector MinDists = BoxMin - LocalPoint;
	FReal Phi;
	if (IsVectorLessEqual(LocalPoint, BoxMax) && IsVectorGreaterEqual(LocalPoint, BoxMin))
	{
		const FVector LocalMax = VectorMax(MinDists, MaxDists);
		Phi = select(LocalMax.V[0] > LocalMax.V[1], 
				select(LocalMax.V[0] > LocalMax.V[2], LocalMax.V[0], LocalMax.V[2]), 
				select(LocalMax.V[1] > LocalMax.V[2], LocalMax.V[1], LocalMax.V[2]));
	}
	else
	{
		FVector Normal;

		Normal.V[0] = select(MaxDists.V[0] > ZERO, MaxDists.V[0], select(MinDists.V[0] > ZERO, -MinDists.V[0], ZERO));
		Normal.V[1] = select(MaxDists.V[1] > ZERO, MaxDists.V[1], select(MinDists.V[1] > ZERO, -MinDists.V[1], ZERO));
		Normal.V[2] = select(MaxDists.V[2] > ZERO, MaxDists.V[2], select(MinDists.V[2] > ZERO, -MinDists.V[2], ZERO));

		Phi = VectorSize(Normal);
	}

	return Phi;
}

export void SampleBoxNormalAverage(const uniform FVector &BoxMin,
									const uniform FVector &BoxMax, 
									const uniform FTransform &SampleToObjectTransform, 
									const uniform FVector SampleParticles[],
									const uniform int PotentialParticles[],
									const uniform FReal Thickness, 
									uniform FReal &TotalThickness, 
									uniform FVector &ConstraintLocation,
									const uniform unsigned int NumParticles)
{
	FVector ConstraintLocationSum = ZeroVector;
	FReal ThicknessSum = 0;

	foreach(i = 0 ... NumParticles)
	{
		FVector Particle = VectorGather(&SampleParticles[PotentialParticles[i]]);

		FVector LocalPoint = TransformPositionNoScale(SampleToObjectTransform, Particle);
		FReal LocalPhi = BoxPhi(BoxMin, BoxMax, LocalPoint);

		FReal LocalThickness = LocalPhi - Thickness;
		if(LocalThickness < -KINDA_SMALL_NUMBER)
		{
			ConstraintLocationSum = ConstraintLocationSum + (LocalPoint * LocalThickness);
			ThicknessSum += LocalThickness;
		}
	}

	ConstraintLocation = ConstraintLocation + VectorReduceAdd(ConstraintLocationSum);
	TotalThickness = TotalThickness + reduce_add(ThicknessSum);
}

export uniform bool SampleBoxNoNormal(const uniform FVector &BoxMin,
									const uniform FVector &BoxMax, 
									const uniform FTransform &SampleToObjectTransform, 
									const uniform FVector SampleParticles[],
									const uniform int PotentialParticles[],
									uniform int &DeepestParticle,
									uniform FReal &Phi,
									const uniform unsigned int NumParticles)
{
	uniform bool bPhiFound = false;

	foreach(i = 0 ... NumParticles)
	{
		const int ParticleNum = PotentialParticles[i];

		FVector Particle = VectorGather(&SampleParticles[ParticleNum]);

		FVector LocalPoint = TransformPositionNoScale(SampleToObjectTransform, Particle);
		FReal LocalPhi = BoxPhi(BoxMin, BoxMax, LocalPoint);

		if(LocalPhi < Phi)
		{
			Phi = reduce_min(LocalPhi);
			foreach_active(j)
			{
				if(extract(LocalPhi, j) == Phi)
				{
					DeepestParticle = extract(ParticleNum, j);
				}
			}
			bPhiFound = true;
		}
	}

	return bPhiFound;
}


export void SampleBoxNormalAverageAll(const uniform FVector &BoxMin,
									const uniform FVector &BoxMax, 
									const uniform FTransform &SampleToObjectTransform, 
									const uniform FVector SampleParticles[],
									const uniform FReal Thickness, 
									uniform FReal &TotalThickness, 
									uniform FVector &ConstraintLocation,
									const uniform unsigned int NumParticles)
{
	FVector ConstraintLocationSum = ZeroVector;
	FReal ThicknessSum = 0;

	foreach(i = 0 ... NumParticles)
	{
		FVector Particle = VectorLoad(&SampleParticles[extract(i, 0)]);

		FVector LocalPoint = TransformPositionNoScale(SampleToObjectTransform, Particle);
		FReal LocalPhi = BoxPhi(BoxMin, BoxMax, LocalPoint);

		FReal LocalThickness = LocalPhi - Thickness;
		if(LocalThickness < -KINDA_SMALL_NUMBER)
		{
			ConstraintLocationSum = ConstraintLocationSum + (LocalPoint * LocalThickness);
			ThicknessSum += LocalThickness;
		}
	}

	ConstraintLocation = ConstraintLocation + VectorReduceAdd(ConstraintLocationSum);
	TotalThickness = TotalThickness + reduce_add(ThicknessSum);
}

export uniform bool SampleBoxNoNormalAll(const uniform FVector &BoxMin,
										const uniform FVector &BoxMax,
										const uniform FTransform &SampleToObjectTransform,
										const uniform FVector SampleParticles[],
										uniform int &DeepestParticle,
										uniform FReal &Phi,
										const uniform unsigned int NumParticles)
{
	uniform bool bPhiFound = false;

	foreach(i = 0 ... NumParticles)
	{
		FVector Particle = VectorLoad(&SampleParticles[extract(i, 0)]);

		FVector LocalPoint = TransformPositionNoScale(SampleToObjectTransform, Particle);
		FReal LocalPhi = BoxPhi(BoxMin, BoxMax, LocalPoint);

		if(LocalPhi < Phi)
		{
			Phi = reduce_min(LocalPhi);
			foreach_active(j)
			{
				if(extract(LocalPhi, j) == Phi)
				{
					DeepestParticle = extract(i, j);
				}
			}
			bPhiFound = true;
		}
	}

	return bPhiFound;
}

inline static FVector ClampMinusHalf(const uniform FVector &Min, const uniform FVector &Max, const FVector& LocalPoint)
{
	FVector Result;

	Result.V[0] = select(LocalPoint.V[0] > Max.V[0], Max.V[0], select(LocalPoint.V[0] < Min.V[0], Min.V[0], LocalPoint.V[0]));
	Result.V[1] = select(LocalPoint.V[1] > Max.V[1], Max.V[1], select(LocalPoint.V[1] < Min.V[1], Min.V[1], LocalPoint.V[1]));
	Result.V[2] = select(LocalPoint.V[2] > Max.V[2], Max.V[2], select(LocalPoint.V[2] < Min.V[2], Min.V[2], LocalPoint.V[2]));

	return Result;
}

inline static FReal GetCell(const uniform TArrayND* uniform LevelSetPhiArray, const FIntVector &Vec)
{
	const int Offset = (Vec.V[0] * LevelSetPhiArray->MCounts.V[1] + Vec.V[1]) * LevelSetPhiArray->MCounts.V[2] + Vec.V[2];

	#pragma ignore warning(perf)
	return LevelSetPhiArray->MArray[Offset];
}

inline static FReal LinearlyInterpolate1D(const FReal& Prev, const FReal& Next, const FReal Alpha)
{
	return Prev + (Next - Prev) * Alpha;
}

inline static FVector Location(const uniform FVector &MinCorner, const uniform FVector &Dx, const FIntVector& X)
{
	return Dx * X + MinCorner + (Dx / 2);
}

inline static FReal LinearlyInterpolate(const uniform FVector &MinCorner,
							const uniform FVector &MaxCorner,
							const uniform FVector &Dx,
							const uniform FIntVector &Cells,
							const uniform TArrayND* uniform LevelSetPhiArray, 
							const FVector& X)
{
	FIntVector XCell = SetIntVector((X - MinCorner) / Dx);
	
	FVector XCenter = Location(MinCorner, Dx, XCell);
	FIntVector CellPrev;
	for (uniform int i = 0; i < 3; i++)
	{
		CellPrev.V[i] = X.V[i] > XCenter.V[i] ? XCell.V[i] : XCell.V[i] - 1;
	}
	FVector Alpha = (X - Location(MinCorner, Dx, CellPrev)) / Dx;
	
	// Clamp correctly when on border
	for (uniform int i = 0; i < 3; i++)
	{
		if (CellPrev.V[i] == -1)
		{
			CellPrev.V[i] = 0;
			Alpha.V[i] = 0;
		}
		if (CellPrev.V[i] == Cells.V[i] - 1)
		{
			CellPrev.V[i] = Cells.V[i] - 2;
			Alpha.V[i] = 1;
		}
	}

	const FReal interpx1 = LinearlyInterpolate1D(GetCell(LevelSetPhiArray, CellPrev), GetCell(LevelSetPhiArray, CellPrev + SetIntVector(1, 0, 0)), Alpha.V[0]);
	const FReal interpx2 = LinearlyInterpolate1D(GetCell(LevelSetPhiArray, CellPrev + SetIntVector(0, 1, 0)), GetCell(LevelSetPhiArray, CellPrev + SetIntVector(1, 1, 0)), Alpha.V[0]);
	const FReal interpx3 = LinearlyInterpolate1D(GetCell(LevelSetPhiArray, CellPrev + SetIntVector(0, 0, 1)), GetCell(LevelSetPhiArray, CellPrev + SetIntVector(1, 0, 1)), Alpha.V[0]);
	const FReal interpx4 = LinearlyInterpolate1D(GetCell(LevelSetPhiArray, CellPrev + SetIntVector(0, 1, 1)), GetCell(LevelSetPhiArray, CellPrev + SetIntVector(1, 1, 1)), Alpha.V[0]);
	const FReal interpy1 = LinearlyInterpolate1D(interpx1, interpx2, Alpha.V[1]);
	const FReal interpy2 = LinearlyInterpolate1D(interpx3, interpx4, Alpha.V[1]);
	return LinearlyInterpolate1D(interpy1, interpy2, Alpha.V[2]);
}

export void SampleLevelSetNormalAverage(const uniform FVector &GridMinCorner,
										const uniform FVector &GridMaxCorner,
										const uniform FVector &GridDx,
										const uniform FIntVector &GridCounts,
										const uniform TArrayND* uniform LevelSetPhiArray,
										const uniform FTransform &SampleToObjectTransform, 
										const uniform FVector SampleParticles[],
										const uniform int PotentialParticles[],
										const uniform FReal Thickness, 
										uniform FReal &TotalThickness, 
										uniform FVector &ConstraintLocation,
										const uniform unsigned int NumParticles)
{
	FVector ConstraintLocationSum = ZeroVector;
	FReal ThicknessSum = 0;

	uniform FVector GridClampMax = GridMaxCorner - GridDx * ONE_HALF;
	uniform FVector GridClampMin = GridMinCorner + GridDx * ONE_HALF;

	foreach(i = 0 ... NumParticles)
	{
		FVector Particle = VectorGather(&SampleParticles[PotentialParticles[i]]);

		FVector LocalPoint = TransformPositionNoScale(SampleToObjectTransform, Particle);
		FVector Location = ClampMinusHalf(GridClampMin, GridClampMax, LocalPoint);
		FReal SizeSquared = VectorSizeSquared(Location - LocalPoint);
		FReal LocalPhi = LinearlyInterpolate(GridMinCorner, GridMaxCorner, GridDx, GridCounts, LevelSetPhiArray, Location);

		if(SizeSquared)
		{
			LocalPhi = rcp(rsqrt(SizeSquared)) + LocalPhi;
		}

		FReal LocalThickness = LocalPhi - Thickness;
		if(LocalThickness < -KINDA_SMALL_NUMBER)
		{
			ConstraintLocationSum = ConstraintLocationSum + (LocalPoint * LocalThickness);
			ThicknessSum += LocalThickness;
		}
	}

	ConstraintLocation = ConstraintLocation + VectorReduceAdd(ConstraintLocationSum);
	TotalThickness = TotalThickness + reduce_add(ThicknessSum);
}

export uniform bool SampleLevelSetNoNormal(const uniform FVector &GridMinCorner,
										const uniform FVector &GridMaxCorner,
										const uniform FVector &GridDx,
										const uniform FIntVector &GridCounts,
										const uniform TArrayND* uniform LevelSetPhiArray,
										const uniform FTransform &SampleToObjectTransform, 
										const uniform FVector SampleParticles[],
										const uniform int PotentialParticles[],
										uniform int &DeepestParticle,
										uniform FReal &Phi,
										const uniform unsigned int NumParticles)
{
	uniform FVector GridClampMax = GridMaxCorner - GridDx * ONE_HALF;
	uniform FVector GridClampMin = GridMinCorner + GridDx * ONE_HALF;

	uniform bool bPhiFound = false;

	foreach(i = 0 ... NumParticles)
	{
		const int ParticleNum = PotentialParticles[i];

		FVector Particle = VectorGather(&SampleParticles[ParticleNum]);

		FVector LocalPoint = TransformPositionNoScale(SampleToObjectTransform, Particle);
		FVector Location = ClampMinusHalf(GridClampMin, GridClampMax, LocalPoint);
		FReal SizeSquared = VectorSizeSquared(Location - LocalPoint);
		FReal LocalPhi = LinearlyInterpolate(GridMinCorner, GridMaxCorner, GridDx, GridCounts, LevelSetPhiArray, Location);

		if(SizeSquared)
		{
			LocalPhi = rcp(rsqrt(SizeSquared)) + LocalPhi;
		}

		if(LocalPhi < Phi)
		{
			Phi = reduce_min(LocalPhi);
			foreach_active(j)
			{
				if(extract(LocalPhi, j) == Phi)
				{
					DeepestParticle = extract(ParticleNum, j);
				}
			}
			bPhiFound = true;
		}
	}

	return bPhiFound;
}

export void SampleLevelSetNormalAverageAll(const uniform FVector &GridMinCorner,
										const uniform FVector &GridMaxCorner,
										const uniform FVector &GridDx,
										const uniform FIntVector &GridCounts,
										const uniform TArrayND* uniform LevelSetPhiArray,
										const uniform FTransform &SampleToObjectTransform, 
										const uniform FVector SampleParticles[],
										const uniform FReal Thickness, 
										uniform FReal &TotalThickness, 
										uniform FVector &ConstraintLocation,
										const uniform unsigned int NumParticles)
{
	FVector ConstraintLocationSum = ZeroVector;
	FReal ThicknessSum = 0;

	uniform FVector GridClampMax = GridMaxCorner - GridDx * ONE_HALF;
	uniform FVector GridClampMin = GridMinCorner + GridDx * ONE_HALF;

	foreach(i = 0 ... NumParticles)
	{
		FVector Particle = VectorLoad(&SampleParticles[extract(i, 0)]);

		FVector LocalPoint = TransformPositionNoScale(SampleToObjectTransform, Particle);
		FVector Location = ClampMinusHalf(GridClampMin, GridClampMax, LocalPoint);
		FReal SizeSquared = VectorSizeSquared(Location - LocalPoint);
		FReal LocalPhi = LinearlyInterpolate(GridMinCorner, GridMaxCorner, GridDx, GridCounts, LevelSetPhiArray, Location);

		if(SizeSquared)
		{
			LocalPhi = rcp(rsqrt(SizeSquared)) + LocalPhi;
		}

		FReal LocalThickness = LocalPhi - Thickness;
		if(LocalThickness < -KINDA_SMALL_NUMBER)
		{
			ConstraintLocationSum = ConstraintLocationSum + (LocalPoint * LocalThickness);
			ThicknessSum += LocalThickness;
		}
	}

	ConstraintLocation = ConstraintLocation + VectorReduceAdd(ConstraintLocationSum);
	TotalThickness = TotalThickness + reduce_add(ThicknessSum);
}

export uniform bool SampleLevelSetNoNormalAll(const uniform FVector &GridMinCorner,
											const uniform FVector &GridMaxCorner,
											const uniform FVector &GridDx,
											const uniform FIntVector &GridCounts,
											const uniform TArrayND* uniform LevelSetPhiArray,
											const uniform FTransform &SampleToObjectTransform, 
											const uniform FVector SampleParticles[],
											uniform int &DeepestParticle,
											uniform FReal &Phi,
											const uniform unsigned int NumParticles)
{
	uniform FVector GridClampMax = GridMaxCorner - GridDx * ONE_HALF;
	uniform FVector GridClampMin = GridMinCorner + GridDx * ONE_HALF;

	uniform bool bPhiFound = false;

	foreach(i = 0 ... NumParticles)
	{
		FVector Particle = VectorLoad(&SampleParticles[extract(i, 0)]);

		FVector LocalPoint = TransformPositionNoScale(SampleToObjectTransform, Particle);
		FVector Location = ClampMinusHalf(GridClampMin, GridClampMax, LocalPoint);
		FReal SizeSquared = VectorSizeSquared(Location - LocalPoint);
		FReal LocalPhi = LinearlyInterpolate(GridMinCorner, GridMaxCorner, GridDx, GridCounts, LevelSetPhiArray, Location);

		if(SizeSquared)
		{
			LocalPhi = rcp(rsqrt(SizeSquared)) + LocalPhi;
		}

		if(LocalPhi < Phi)
		{
			Phi = reduce_min(LocalPhi);
			foreach_active(j)
			{
				if(extract(LocalPhi, j) == Phi)
				{
					DeepestParticle = extract(i, j);
				}
			}
			bPhiFound = true;
		}
	}

	return bPhiFound;
}

export void SamplePlaneNormalAverageAll(const uniform FVector &ObjectNormal,
									const uniform FVector &ObjectMX,
									const uniform FTransform &SampleToObjectTransform, 
									const uniform FVector SampleParticles[],
									const uniform FReal Thickness, 
									uniform FReal &TotalThickness, 
									uniform FVector &ConstraintLocation,
									const uniform unsigned int NumParticles)
{
	FVector ConstraintLocationSum = ZeroVector;
	FReal ThicknessSum = 0;

	foreach(i = 0 ... NumParticles)
	{
		FVector Particle = VectorLoad(&SampleParticles[extract(i, 0)]);

		FVector LocalPoint = TransformPositionNoScale(SampleToObjectTransform, Particle);
		FReal LocalPhi = VectorDot(LocalPoint - ObjectMX, ObjectNormal);

		FReal LocalThickness = LocalPhi - Thickness;
		if(LocalThickness < -KINDA_SMALL_NUMBER)
		{
			ConstraintLocationSum = ConstraintLocationSum + (LocalPoint * LocalThickness);
			ThicknessSum += LocalThickness;
		}
	}

	ConstraintLocation = ConstraintLocation + VectorReduceAdd(ConstraintLocationSum);
	TotalThickness = TotalThickness + reduce_add(ThicknessSum);
}

export uniform bool SamplePlaneNoNormalAll(const uniform FVector &ObjectNormal,
										const uniform FVector &ObjectMX,
										const uniform FTransform &SampleToObjectTransform, 
										const uniform FVector SampleParticles[],
										uniform int &DeepestParticle,
										uniform FReal &Phi,
										const uniform unsigned int NumParticles)
{
	uniform bool bPhiFound = false;

	foreach(i = 0 ... NumParticles)
	{
		FVector Particle = VectorLoad(&SampleParticles[extract(i, 0)]);

		FVector LocalPoint = TransformPositionNoScale(SampleToObjectTransform, Particle);
		FReal LocalPhi = VectorDot(LocalPoint - ObjectMX, ObjectNormal);

		if(LocalPhi < Phi)
		{
			Phi = reduce_min(LocalPhi);
			foreach_active(j)
			{
				if(extract(LocalPhi, j) == Phi)
				{
					DeepestParticle = extract(i, j);
				}
			}
			bPhiFound = true;
		}
	}

	return bPhiFound;
}

export void SampleSphereNormalAverageAll(const uniform FReal &ObjectRadius,
									const uniform FVector &ObjectMX,
									const uniform FTransform &SampleToObjectTransform, 
									const uniform FVector SampleParticles[],
									const uniform FReal Thickness, 
									uniform FReal &TotalThickness, 
									uniform FVector &ConstraintLocation,
									const uniform unsigned int NumParticles)
{
	FVector ConstraintLocationSum = ZeroVector;
	FReal ThicknessSum = 0;

	foreach(i = 0 ... NumParticles)
	{
		FVector Particle = VectorLoad(&SampleParticles[extract(i, 0)]);

		FVector LocalPoint = TransformPositionNoScale(SampleToObjectTransform, Particle);
		FReal LocalPhi = ObjectRadius - VectorSize(LocalPoint - ObjectMX);

		FReal LocalThickness = LocalPhi - Thickness;
		if(LocalThickness < -KINDA_SMALL_NUMBER)
		{
			ConstraintLocationSum = ConstraintLocationSum + (LocalPoint * LocalThickness);
			ThicknessSum += LocalThickness;
		}
	}

	ConstraintLocation = ConstraintLocation + VectorReduceAdd(ConstraintLocationSum);
	TotalThickness = TotalThickness + reduce_add(ThicknessSum);
}

export uniform bool SampleSphereNoNormalAll(const uniform FReal &ObjectRadius,
										const uniform FVector &ObjectMX,
										const uniform FTransform &SampleToObjectTransform, 
										const uniform FVector SampleParticles[],
										uniform int &DeepestParticle,
										uniform FReal &Phi,
										const uniform unsigned int NumParticles)
{
	uniform bool bPhiFound = false;

	foreach(i = 0 ... NumParticles)
	{
		FVector Particle = VectorLoad(&SampleParticles[extract(i, 0)]);

		FVector LocalPoint = TransformPositionNoScale(SampleToObjectTransform, Particle);
		FReal LocalPhi = ObjectRadius - VectorSize(LocalPoint - ObjectMX);

		if(LocalPhi < Phi)
		{
			Phi = reduce_min(LocalPhi);
			foreach_active(j)
			{
				if(extract(LocalPhi, j) == Phi)
				{
					DeepestParticle = extract(i, j);
				}
			}
			bPhiFound = true;
		}
	}

	return bPhiFound;
}


//
//
///////////////////////////////////////////////////////////////////////////////
//
//

// Get the CoM position given the Actor position
static inline uniform FVector GetCoMWorldPosition(
	const uniform FVector& PActorWorld,
	const uniform FVector4& QActorWorld,
	const uniform FVector& PCoMLocal)
{
	return PActorWorld + VectorQuaternionRotateVector(QActorWorld, PCoMLocal);
}

// Get the CoM rotation given the Actor rotation
static inline uniform FVector4 GetCoMWorldRotation(
	const uniform FVector4& QActorWorld,
	const uniform FVector4& QCoMLocal)
{
	return VectorQuaternionMultiply2(QActorWorld, QCoMLocal);
}

// Set the Actor position and rotation given the CoM position and rotation
static inline uniform FVector SetCoMWorldTransform(
	uniform FVector& PActorWorld,
	uniform FVector4& QActorWorld,
	const uniform FVector& PCoMWorld,
	const uniform FVector4& QCoMWorld,
	const uniform FVector& PCoMLocal,
	const uniform FVector4& QCoMLocal)
{
	uniform FVector4 QCoMLocalInverse = QuatInverse(QCoMLocal);
	QActorWorld = VectorQuaternionMultiply2(QCoMWorld, QCoMLocalInverse);
	PActorWorld = PCoMWorld - VectorQuaternionRotateVector(QActorWorld, PCoMLocal);
}

// @todo(ccaulfield): dedupe
static inline uniform FMatrix33 ComputeWorldSpaceInertia(const uniform FVector4 &Q, const uniform FVector &I)
{
	const uniform FMatrix33 QM = MakeQuatRotationTranslationMatrix(Q);
	const uniform FMatrix33 L = SetMatrix33(I.V[0], I.V[1], I.V[2]);
	return MultiplyAB(QM, MultiplyABt(L, QM));
}

// @todo(ccaulfield): dedupe
static inline uniform FMatrix33 ComputeJointFactorMatrix(const uniform FVector& V, const uniform FMatrix33& M, const uniform FReal Im)
{
	// Rigid objects rotational contribution to the impulse.
	// Vx*M*VxT+Im
	const uniform FVector V221 = SetVector(-V.V[2], V.V[2], -V.V[1]);
	const uniform FVector V222 = SetVector(-V.V[2], -V.V[2], -V.V[2]);
	const uniform FVector M544 = SetVector(M.M[5], M.M[4], M.M[4]);
	const uniform FVector V111 = SetVector(V.V[1], V.V[1], V.V[1]);
	const uniform FVector M988 = SetVector(M.M[9], M.M[8], M.M[8]);
	const uniform FVector V100 = SetVector(V.V[1], -V.V[0], V.V[0]);
	const uniform FVector M995 = SetVector(M.M[9], M.M[9], M.M[5]);
	const uniform FVector M10109 = SetVector(M.M[10], M.M[10], M.M[9]);
	const uniform FVector IM = SetVector(Im, 0, 0);

	const uniform FVector R1 = (V221 * (V222 * M544 + V111 * M988)) + (V100 * (V222 * M995 + V111 * M10109)) + IM;

	const uniform FVector V211 = SetVector(V.V[2], -V.V[1], -V.V[1]);
	const uniform FVector V221_2 = SetVector(V.V[2], V.V[2], -V.V[1]);
	const uniform FVector M000 = SetVector(M.M[0], M.M[0], M.M[0]);
	const uniform FVector V000 = SetVector(-V.V[0], -V.V[0], V.V[0]);
	const uniform FVector M884 = SetVector(M.M[8], M.M[8], M.M[4]);
	const uniform FVector V000_2 = SetVector(-V.V[0], V.V[0], V.V[0]);
	const uniform FVector M844 = SetVector(M.M[8], M.M[4], M.M[4]);
	const uniform FVector M1095 = SetVector(M.M[10], M.M[9], M.M[5]);
	const uniform FVector IM_2 = SetVector(Im, 0, Im);

	const uniform FVector R2 = (V211 * (V221_2 * M000 + V000 * M884)) + (V000_2 * (V221_2 * M844 + V000 * M1095)) + IM_2;

	return SetMatrix33(R1.V[0], R1.V[1], R1.V[2], R2.V[0], R2.V[1], R2.V[2]);
}

