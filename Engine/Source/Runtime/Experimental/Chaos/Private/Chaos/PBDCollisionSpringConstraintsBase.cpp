// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/PBDCollisionSpringConstraintsBase.h"
#include "Chaos/Plane.h"
#include "Chaos/Triangle.h"
#include "Chaos/TriangleCollisionPoint.h"
#include "Chaos/TriangleMesh.h"
#include "Chaos/PBDSoftsSolverParticles.h"
#include "Chaos/SoftsEvolutionLinearSystem.h"
#include "Chaos/SoftsSolverParticlesRange.h"

#if !COMPILE_WITHOUT_UNREAL_SUPPORT
#include "Chaos/Framework/Parallel.h"
#include <atomic>

namespace Chaos::Softs {

static TConstArrayView<FSolverVec3> GetConstArrayView(const FSolverParticles& Particles, int32 Offset, int32 NumParticles, const TArray<FSolverVec3>* const Data)
{
	if (Data)
	{
		return TConstArrayView<FSolverVec3>(Data->GetData(), NumParticles + Offset);
	}
	return TConstArrayView<FSolverVec3>();
}

static TConstArrayView<FSolverVec3> GetConstArrayView(const FSolverParticlesRange& Particles, int32 Offset, int32 NumParticles, const TArray<FSolverVec3>* const Data)
{
	if (Data)
	{
		return Particles.GetConstArrayView(*Data);
	}
	return TConstArrayView<FSolverVec3>();
}

FPBDCollisionSpringConstraintsBase::FPBDCollisionSpringConstraintsBase(
	const int32 InOffset,
	const int32 InNumParticles,
	const FTriangleMesh& InTriangleMesh,
	const TArray<FSolverVec3>* InReferencePositions,
	TSet<TVec2<int32>>&& InDisabledCollisionElements,
	const TConstArrayView<FRealSingle>& InThicknessMultipliers,
	const TConstArrayView<FRealSingle>& InKinematicColliderFrictionMultipliers,
	const TConstArrayView<int32>& InSelfCollisionLayers,
	const FSolverVec2 InThickness,
	const FSolverReal InStiffness,
	const FSolverReal InFrictionCoefficient,
	const bool bInOnlyCollideKinematics,
	const FSolverReal InKinematicColliderThickness,
	const FSolverReal InKinematicColliderStiffness,
	const FSolverVec2 InKinematicColliderFrictionCoefficient,
	const FSolverReal InProximityStiffness)
	: ThicknessWeighted(InThickness, InThicknessMultipliers, InNumParticles)
	, Stiffness(InStiffness)
	, FrictionCoefficient(InFrictionCoefficient)
	, bOnlyCollideKinematics(bInOnlyCollideKinematics)
	, KinematicColliderFrictionCoefficient(InKinematicColliderFrictionCoefficient, InKinematicColliderFrictionMultipliers, InNumParticles)
	, KinematicCollisions(InNumParticles, ThicknessWeighted, KinematicColliderFrictionCoefficient, InKinematicColliderStiffness, InKinematicColliderThickness)
	, ProximityStiffness(InProximityStiffness)
	, TriangleMesh(InTriangleMesh)
	, ReferencePositions(InReferencePositions)
	, DisabledCollisionElements(InDisabledCollisionElements)
	, Offset(InOffset)
	, NumParticles(InNumParticles)
	, bGlobalIntersectionAnalysis(false)
{
	UpdateCollisionLayers(InSelfCollisionLayers);
}

void FPBDCollisionSpringConstraintsBase::UpdateCollisionLayers(const TConstArrayView<int32>& InFaceCollisionLayers)
{
	if (InFaceCollisionLayers.Num() != TriangleMesh.GetElements().Num())
	{
		// Reset collision layers
		FaceCollisionLayers = TConstArrayView<int32>();
		VertexCollisionLayers.Reset();
	}
	else
	{
		FaceCollisionLayers = InFaceCollisionLayers;
		VertexCollisionLayers.SetNumUninitialized(NumParticles);

		TConstArrayView<TArray<int32>> PointToTriangle = TriangleMesh.GetPointToTriangleMap();
		for (int32 ParticleIndexNoOffset = 0; ParticleIndexNoOffset < NumParticles; ++ParticleIndexNoOffset)
		{
			const int32 ParticleIndex = ParticleIndexNoOffset + Offset;
			TVec2<int32>& VertexCollisionLayer = VertexCollisionLayers[ParticleIndexNoOffset];
			VertexCollisionLayer = TVec2<int32>(INDEX_NONE);
			for (const int32 FaceIndex : PointToTriangle[ParticleIndex])
			{
				if (FaceCollisionLayers[FaceIndex] != INDEX_NONE)
				{
					VertexCollisionLayer[0] = VertexCollisionLayer[0] == INDEX_NONE ? 
						FaceCollisionLayers[FaceIndex] : FMath::Min(FaceCollisionLayers[FaceIndex], VertexCollisionLayer[0]);

					VertexCollisionLayer[1] = VertexCollisionLayer[1] == INDEX_NONE ?
						FaceCollisionLayers[FaceIndex] : FMath::Max(FaceCollisionLayers[FaceIndex], VertexCollisionLayer[1]);
				}
			}
		}
	}
}

template<typename SpatialAccelerator, typename SolverParticlesOrRange>
void FPBDCollisionSpringConstraintsBase::Init(const SolverParticlesOrRange& Particles, const SpatialAccelerator& Spatial, const TConstArrayView<FPBDTriangleMeshCollisions::FGIAColor>& VertexGIAColors, const TArray<FPBDTriangleMeshCollisions::FGIAColor>& TriangleGIAColors)
{
	FPBDTriangleMeshCollisions::FTriangleSubMesh SubMesh(TriangleMesh);
	SubMesh.InitAllDynamic();
	SpatialAccelerator UnusedKinematicSpatial;
	constexpr FSolverReal LargeDt = UE_BIG_NUMBER; // This will disable all kinematic collider timers.
	Init(Particles, LargeDt, SubMesh, Spatial, UnusedKinematicSpatial, VertexGIAColors, TriangleGIAColors);
}
template void CHAOS_API FPBDCollisionSpringConstraintsBase::Init<FTriangleMesh::TBVHType<FSolverReal>>(const FSolverParticles& Particles, const FTriangleMesh::TBVHType<FSolverReal>& Spatial, 
	const TConstArrayView<FPBDTriangleMeshCollisions::FGIAColor>& VertexGIAColors, const TArray<FPBDTriangleMeshCollisions::FGIAColor>& TriangleGIAColors);
template void CHAOS_API FPBDCollisionSpringConstraintsBase::Init<FTriangleMesh::TSpatialHashType<FSolverReal>>(const FSolverParticles& Particles, const FTriangleMesh::TSpatialHashType<FSolverReal>& Spatial,
	const TConstArrayView<FPBDTriangleMeshCollisions::FGIAColor>& VertexGIAColors, const TArray<FPBDTriangleMeshCollisions::FGIAColor>& TriangleGIAColors);
template void CHAOS_API FPBDCollisionSpringConstraintsBase::Init<FTriangleMesh::TBVHType<FSolverReal>>(const FSolverParticlesRange& Particles, const FTriangleMesh::TBVHType<FSolverReal>& Spatial,
	const TConstArrayView<FPBDTriangleMeshCollisions::FGIAColor>& VertexGIAColors, const TArray<FPBDTriangleMeshCollisions::FGIAColor>& TriangleGIAColors);
template void CHAOS_API FPBDCollisionSpringConstraintsBase::Init<FTriangleMesh::TSpatialHashType<FSolverReal>>(const FSolverParticlesRange& Particles, const FTriangleMesh::TSpatialHashType<FSolverReal>& Spatial,
	const TConstArrayView<FPBDTriangleMeshCollisions::FGIAColor>& VertexGIAColors, const TArray<FPBDTriangleMeshCollisions::FGIAColor>& TriangleGIAColors);

template<typename SpatialAccelerator, typename SolverParticlesOrRange>
void FPBDCollisionSpringConstraintsBase::Init(const SolverParticlesOrRange& Particles, const FSolverReal Dt, const FPBDTriangleMeshCollisions::FTriangleSubMesh& CollidableSubMesh,
	const SpatialAccelerator& DynamicSpatial, const SpatialAccelerator& KinematicColliderSpatial, const TConstArrayView<FPBDTriangleMeshCollisions::FGIAColor>& VertexGIAColors, const TArray<FPBDTriangleMeshCollisions::FGIAColor>& TriangleGIAColors)
{

	if constexpr (std::is_same<SpatialAccelerator, THierarchicalSpatialHash<int32, FSolverReal>>::value && std::is_same<FSolverParticlesRange, SolverParticlesOrRange>::value)
	{
		KinematicCollisions.SetGeometry(CollidableSubMesh.GetKinematicColliderSubMesh(), Particles.XArray(), Particles.GetPAndInvM(), Particles.GetV(), KinematicColliderSpatial);
		KinematicCollisions.Init(Particles, Dt);
	}
	else
	{
		// Kinematic collider only supports spatial hash type
		KinematicCollisions.Reset();
	}

	const int32 NumDynamicElements = CollidableSubMesh.GetDynamicSubMesh().GetNumElements();

	if (!NumDynamicElements || bOnlyCollideKinematics)
	{
		Constraints.Reset();
		Barys.Reset();
		FlipNormal.Reset();
		ConstraintTypes.Reset();
		return;
	}
	{

		bGlobalIntersectionAnalysis = VertexGIAColors.Num() == NumParticles + Offset && TriangleGIAColors.Num() == NumDynamicElements;
		TRACE_CPUPROFILER_EVENT_SCOPE(ChaosPBDCollisionSpring_ProximityQuery);

		const int32 NumCollidableParticles = CollidableSubMesh.GetDynamicVertices().IsEmpty() ? NumParticles : CollidableSubMesh.GetDynamicVertices().Num();

		// Preallocate enough space for all possible connections.
		constexpr int32 MaxConnectionsPerPoint = 3;
		Constraints.SetNumUninitialized(NumCollidableParticles * MaxConnectionsPerPoint);
		Barys.SetNumUninitialized(NumCollidableParticles * MaxConnectionsPerPoint);
		FlipNormal.SetNumUninitialized(NumCollidableParticles * MaxConnectionsPerPoint);
		ConstraintTypes.SetNumUninitialized(NumCollidableParticles * MaxConnectionsPerPoint);

		std::atomic<int32> ConstraintIndex(0);

		const TConstArrayView<FSolverVec3> ReferencePositionsView = GetConstArrayView(Particles, Offset, NumParticles, ReferencePositions);

		PhysicsParallelFor(NumCollidableParticles,
			[this, Dt, &CollidableSubMesh, &DynamicSpatial, &Particles, &ConstraintIndex, MaxConnectionsPerPoint, &VertexGIAColors, &TriangleGIAColors, &ReferencePositionsView](int32 CollidableIndex)
			{
				const int32 i = CollidableSubMesh.GetDynamicVertices().IsEmpty() ? CollidableIndex : CollidableSubMesh.GetDynamicVertices()[CollidableIndex] - Offset; // DynamicVertices already has offset applied.
				const int32 Index = i + Offset;
				if (Particles.InvM(Index) == (FSolverReal)0.)
				{
					return;
				}
				constexpr FSolverReal ExtraThicknessMult = 1.5f;

				const bool bVertexHasCollisionLayers = VertexCollisionLayers.IsValidIndex(i) && VertexCollisionLayers[i][0] != INDEX_NONE;
				check(!bVertexHasCollisionLayers || VertexCollisionLayers[i][0] <= VertexCollisionLayers[i][1]);

				const FSolverReal ParticleThickness = ThicknessWeighted.GetValue(i);

				// Dynamic collisions
				TArray< TTriangleCollisionPoint<FSolverReal> > DynamicResult;

				auto BroadphaseTest = [this, bVertexHasCollisionLayers, &Particles, &CollidableSubMesh, &VertexGIAColors, &TriangleGIAColors](const int32 PointIndex, const int32 SubMeshTriangleIndex)->bool
				{
					const TVector<int32, 3>& Elem = CollidableSubMesh.GetDynamicSubMesh().GetElements()[SubMeshTriangleIndex];
					const int32 FullMeshTriangleIndex = CollidableSubMesh.GetFullMeshElementIndexFromDynamicElement(SubMeshTriangleIndex);

					bool bUseCollisionLayerOverride = false;
					if (bVertexHasCollisionLayers && FaceCollisionLayers[FullMeshTriangleIndex] != INDEX_NONE)
					{
						if (FaceCollisionLayers[FullMeshTriangleIndex] < VertexCollisionLayers[PointIndex - Offset][0] || FaceCollisionLayers[FullMeshTriangleIndex] > VertexCollisionLayers[PointIndex - Offset][1])
						{
							bUseCollisionLayerOverride = true;
						}
					}
					if (!bUseCollisionLayerOverride && bGlobalIntersectionAnalysis)
					{
						const bool bIsAnyBoundary = VertexGIAColors[PointIndex].IsBoundary()
							|| VertexGIAColors[Elem[0]].IsBoundary()
							|| VertexGIAColors[Elem[1]].IsBoundary()
							|| VertexGIAColors[Elem[2]].IsBoundary();
						if (bIsAnyBoundary)
						{
							return false;
						}

						const bool bAreBothLoop = VertexGIAColors[PointIndex].IsLoop() &&
							(VertexGIAColors[Elem[0]].IsLoop()
								|| VertexGIAColors[Elem[1]].IsLoop()
								|| VertexGIAColors[Elem[2]].IsLoop()
								|| TriangleGIAColors[SubMeshTriangleIndex].IsLoop());

						if (bAreBothLoop)
						{
							return false;
						}
					}

					if (DisabledCollisionElements.Contains({ PointIndex, Elem[0] }) ||
						DisabledCollisionElements.Contains({ PointIndex, Elem[1] }) ||
						DisabledCollisionElements.Contains({ PointIndex, Elem[2] }))
					{
						return false;
					}

					return true;
				};

				if (ThicknessWeighted.HasWeightMap())
				{
					if constexpr (std::is_same<SpatialAccelerator, THierarchicalSpatialHash<int32, FSolverReal>>::value)
					{
						CollidableSubMesh.GetDynamicSubMesh().PointProximityQuery(DynamicSpatial, static_cast<const TArrayView<const FSolverVec3>&>(Particles.XArray()), Index, Particles.X(Index), ParticleThickness * ExtraThicknessMult, ThicknessWeighted, ExtraThicknessMult, Offset, BroadphaseTest, DynamicResult);
					}
					else
					{
						check(false);
					}
				}
				else
				{
					CollidableSubMesh.GetDynamicSubMesh().PointProximityQuery(DynamicSpatial, static_cast<const TArrayView<const FSolverVec3>&>(Particles.XArray()), Index, Particles.X(Index), ParticleThickness * ExtraThicknessMult, (FSolverReal)ThicknessWeighted * ExtraThicknessMult, BroadphaseTest, DynamicResult);
				}

				if ( DynamicResult.Num() )
				{
					if (DynamicResult.Num() > MaxConnectionsPerPoint)
					{
						// TODO: once we have a PartialSort, use that instead here.
						DynamicResult.Sort(
							[](const TTriangleCollisionPoint<FSolverReal>& First, const TTriangleCollisionPoint<FSolverReal>& Second)->bool
							{
								return First.Phi < Second.Phi;
							}
						);
						DynamicResult.SetNum(MaxConnectionsPerPoint, EAllowShrinking::No);
					}

					for (const TTriangleCollisionPoint<FSolverReal>& CollisionPoint : DynamicResult)
					{
						const TVector<int32, 3>& Elem = CollidableSubMesh.GetDynamicSubMesh().GetElements()[CollisionPoint.Indices[1]];
						if (ReferencePositionsView.Num())
						{
							const FSolverVec3& RefP = ReferencePositionsView[Index];
							const FSolverVec3& RefP0 = ReferencePositionsView[Elem[0]];
							const FSolverVec3& RefP1 = ReferencePositionsView[Elem[1]];
							const FSolverVec3& RefP2 = ReferencePositionsView[Elem[2]];
							const FSolverVec3 RefDiff = RefP - CollisionPoint.Bary[1] * RefP0 - CollisionPoint.Bary[2] * RefP1 - CollisionPoint.Bary[3] * RefP2;
							const FSolverReal TriangleThickness = ThicknessWeighted.HasWeightMap() ?
								CollisionPoint.Bary[1] * ThicknessWeighted.GetValue(Elem[0] - Offset) + CollisionPoint.Bary[2] * ThicknessWeighted.GetValue(Elem[1] - Offset) + CollisionPoint.Bary[3] * ThicknessWeighted.GetValue(Elem[2] - Offset) : (FSolverReal)ThicknessWeighted;

							if (RefDiff.SizeSquared() < FMath::Square(ParticleThickness + TriangleThickness))
							{
								continue;
							}
						}

						FSolverVec3 Bary(CollisionPoint.Bary[1], CollisionPoint.Bary[2], CollisionPoint.Bary[3]);

						const int32 FullMeshTriangleIndex = CollidableSubMesh.GetFullMeshElementIndexFromDynamicElement(CollisionPoint.Indices[1]);

						bool bFlipNormal = false;
						EConstraintType ConstraintType = EConstraintType::Default;

						// Check collision layers
						bool bUseCollisionLayerOverride = false;
						if (bVertexHasCollisionLayers && FaceCollisionLayers[FullMeshTriangleIndex] != INDEX_NONE)
						{
							if (FaceCollisionLayers[FullMeshTriangleIndex] < VertexCollisionLayers[i][0])
							{
								// Face is lower layer than the vertex. Vertex should always be in front of face (as UE sees it).
								// NOTE: Chaos internal winding order for normals is reversed, so flip normal in this case.
								bFlipNormal = true;
								bUseCollisionLayerOverride = true;
							}
							else if (FaceCollisionLayers[FullMeshTriangleIndex] > VertexCollisionLayers[i][1])
							{
								// Face is higher layer than the vertex. Vertex should always be behind face (as UE sees it).
								// NOTE: Chaos internal winding order for normals is reversed, so don't flip normal in this case.
								bFlipNormal = false;
								bUseCollisionLayerOverride = true;
							}
						}

						if (!bUseCollisionLayerOverride)
						{
							// NOTE: CollisionPoint.Normal has already been flipped to point toward the Point, so need to recalculate here.
							const TTriangle<FSolverReal> Triangle(Particles.GetX(Elem[0]), Particles.GetX(Elem[1]), Particles.GetX(Elem[2]));
							bFlipNormal = (Particles.GetX(Index) - CollisionPoint.Location).Dot(Triangle.GetNormal()) < 0; // Is Point currently behind Triangle?
							// Doing a check against ANY (plus the TriangleGIAColors which captures sub-triangle intersections) seems to work better than checking against ALL vertex colors where the triangle must agree.
							// In particular, it's better at handling thin regions of intersection where a single vertex or line of vertices intersect through faces.
							// Want Point to push to opposite side of triangle
							if (bGlobalIntersectionAnalysis &&
								(FPBDTriangleMeshCollisions::FGIAColor::ShouldFlipNormal(VertexGIAColors[Index], VertexGIAColors[Elem[0]]) ||
									FPBDTriangleMeshCollisions::FGIAColor::ShouldFlipNormal(VertexGIAColors[Index], VertexGIAColors[Elem[1]]) ||
									FPBDTriangleMeshCollisions::FGIAColor::ShouldFlipNormal(VertexGIAColors[Index], VertexGIAColors[Elem[2]]) ||
									FPBDTriangleMeshCollisions::FGIAColor::ShouldFlipNormal(VertexGIAColors[Index], TriangleGIAColors[CollisionPoint.Indices[1]])))
							{

								bFlipNormal = !bFlipNormal;
								ConstraintType = EConstraintType::GIAFlipped;
							}
						}
						const int32 IndexToWrite = ConstraintIndex.fetch_add(1);

						Constraints[IndexToWrite] = { Index, Elem[0], Elem[1], Elem[2] };
						Barys[IndexToWrite] = Bary;
						FlipNormal[IndexToWrite] = bFlipNormal;
						ConstraintTypes[IndexToWrite] = ConstraintType;
					}
				}
			}
		);

		// Shrink the arrays to the actual number of found constraints.
		const int32 ConstraintNum = ConstraintIndex.load();
		Constraints.SetNum(ConstraintNum, EAllowShrinking::No);
		Barys.SetNum(ConstraintNum, EAllowShrinking::No);
		FlipNormal.SetNum(ConstraintNum, EAllowShrinking::No);
		ConstraintTypes.SetNum(ConstraintNum, EAllowShrinking::No);
	}
}
template void CHAOS_API FPBDCollisionSpringConstraintsBase::Init<FTriangleMesh::TBVHType<FSolverReal>>(const FSolverParticles& Particles, const FSolverReal Dt,
	const FPBDTriangleMeshCollisions::FTriangleSubMesh& CollidableSubMesh, const FTriangleMesh::TBVHType<FSolverReal>& DynamicSpatial, const FTriangleMesh::TBVHType<FSolverReal>& KinematicColliderSpatial,
	const TConstArrayView<FPBDTriangleMeshCollisions::FGIAColor>& VertexGIAColors, const TArray<FPBDTriangleMeshCollisions::FGIAColor>& TriangleGIAColors);
template void CHAOS_API FPBDCollisionSpringConstraintsBase::Init<FTriangleMesh::TSpatialHashType<FSolverReal>>(const FSolverParticles& Particles, const FSolverReal Dt,
	const FPBDTriangleMeshCollisions::FTriangleSubMesh& CollidableSubMesh, const FTriangleMesh::TSpatialHashType<FSolverReal>& DynamicSpatial, const FTriangleMesh::TSpatialHashType<FSolverReal>& KinematicColliderSpatial,
	const TConstArrayView<FPBDTriangleMeshCollisions::FGIAColor>& VertexGIAColors, const TArray<FPBDTriangleMeshCollisions::FGIAColor>& TriangleGIAColors);
template void CHAOS_API FPBDCollisionSpringConstraintsBase::Init<FTriangleMesh::TBVHType<FSolverReal>>(const FSolverParticlesRange& Particles, const FSolverReal Dt,
	const FPBDTriangleMeshCollisions::FTriangleSubMesh& CollidableSubMesh, const FTriangleMesh::TBVHType<FSolverReal>& DynamicSpatial, const FTriangleMesh::TBVHType<FSolverReal>& KinematicColliderSpatial,
	const TConstArrayView<FPBDTriangleMeshCollisions::FGIAColor>& VertexGIAColors, const TArray<FPBDTriangleMeshCollisions::FGIAColor>& TriangleGIAColors);
template void CHAOS_API FPBDCollisionSpringConstraintsBase::Init<FTriangleMesh::TSpatialHashType<FSolverReal>>(const FSolverParticlesRange& Particles, const FSolverReal Dt,
	const FPBDTriangleMeshCollisions::FTriangleSubMesh& CollidableSubMesh, const FTriangleMesh::TSpatialHashType<FSolverReal>& DynamicSpatial, const FTriangleMesh::TSpatialHashType<FSolverReal>& KinematicColliderSpatial,
	const TConstArrayView<FPBDTriangleMeshCollisions::FGIAColor>& VertexGIAColors, const TArray<FPBDTriangleMeshCollisions::FGIAColor>& TriangleGIAColors);

template<typename SolverParticlesOrRange>
FSolverVec3 FPBDCollisionSpringConstraintsBase::GetDelta(const SolverParticlesOrRange& Particles, const int32 ConstraintIndex) const
{
	const TVec4<int32>& Constraint = Constraints[ConstraintIndex];
	const int32 Index1 = Constraint[0];
	const int32 Index2 = Constraint[1];
	const int32 Index3 = Constraint[2];
	const int32 Index4 = Constraint[3];

	const FSolverReal TrianglePointInvM =
		Particles.InvM(Index2) * Barys[ConstraintIndex][0] +
		Particles.InvM(Index3) * Barys[ConstraintIndex][1] +
		Particles.InvM(Index4) * Barys[ConstraintIndex][2];

	const FSolverReal CombinedMass = Particles.InvM(Index1) + TrianglePointInvM;
	if (CombinedMass <= (FSolverReal)1e-7)
	{
		return FSolverVec3(0);
	}

	const FSolverVec3& P1 = Particles.P(Index1);
	const FSolverVec3& P2 = Particles.P(Index2);
	const FSolverVec3& P3 = Particles.P(Index3);
	const FSolverVec3& P4 = Particles.P(Index4);

	const FSolverReal Height = GetConstraintThickness(ConstraintIndex);

	const TTriangle<FSolverReal> Triangle(P2, P3, P4);
	const FSolverVec3 Normal = FlipNormal[ConstraintIndex] ? -Triangle.GetNormal() : Triangle.GetNormal();

	const FSolverVec3& Bary = Barys[ConstraintIndex];
	const FSolverVec3 P = Bary[0] * P2 + Bary[1] * P3 + Bary[2] * P4;
	const FSolverVec3 Difference = P1 - P;
	const FSolverReal NormalDifference = Difference.Dot(Normal);

	// Normal repulsion with friction
	if (NormalDifference > Height)
	{
		return FSolverVec3(0);
	}

	const FSolverReal ConstraintFriction = GetConstraintFrictionCoefficient(ConstraintIndex);

	const FSolverReal NormalDelta = Height - NormalDifference;
	const FSolverVec3 RepulsionDelta = Stiffness * NormalDelta * Normal / CombinedMass;

	if (ConstraintFriction > 0)
	{
		const FSolverVec3& X1 = Particles.GetX(Index1);
		const FSolverVec3 X = Bary[0] * Particles.GetX(Index2) + Bary[1] * Particles.GetX(Index3) + Bary[2] * Particles.GetX(Index4);
		const FSolverVec3 RelativeDisplacement = (P1 - X1) - (P - X) + (Particles.InvM(Index1) - TrianglePointInvM) * RepulsionDelta;
		const FSolverVec3 RelativeDisplacementTangent = RelativeDisplacement - RelativeDisplacement.Dot(Normal) * Normal;
		const FSolverReal RelativeDisplacementTangentLength = RelativeDisplacementTangent.Length();
		const FSolverReal PositionCorrection = FMath::Min(NormalDelta * ConstraintFriction, RelativeDisplacementTangentLength);
		const FSolverReal CorrectionRatio = RelativeDisplacementTangentLength < UE_SMALL_NUMBER ? 0.f : PositionCorrection / RelativeDisplacementTangentLength;
		const FSolverVec3 FrictionDelta = -CorrectionRatio * RelativeDisplacementTangent / CombinedMass;
		return RepulsionDelta + FrictionDelta;
	}
	else
	{
		return RepulsionDelta;
	}
}
template CHAOS_API FSolverVec3 FPBDCollisionSpringConstraintsBase::GetDelta(const FSolverParticles& Particles, const int32 i) const;
template CHAOS_API FSolverVec3 FPBDCollisionSpringConstraintsBase::GetDelta(const FSolverParticlesRange& Particles, const int32 i) const;

void FPBDCollisionSpringConstraintsBase::UpdateLinearSystem(const FSolverParticlesRange& Particles, const FSolverReal Dt, FEvolutionLinearSystem& LinearSystem) const
{
	LinearSystem.ReserveForParallelAdd(Constraints.Num() * 4, Constraints.Num() * 3);
	for (int32 Index = 0; Index < Constraints.Num(); ++Index)
	{
		const TVector<int32, 4>& Constraint = Constraints[Index];
		const int32 Index1 = Constraint[0];
		const int32 Index2 = Constraint[1];
		const int32 Index3 = Constraint[2];
		const int32 Index4 = Constraint[3];
		const FSolverVec3& P1 = Particles.P(Index1);
		const FSolverVec3& P2 = Particles.P(Index2);
		const FSolverVec3& P3 = Particles.P(Index3);
		const FSolverVec3& P4 = Particles.P(Index4);

		const FSolverReal Height = GetConstraintThickness(Index);
		const FSolverVec3 P = Barys[Index][0] * P2 + Barys[Index][1] * P3 + Barys[Index][2] * P4;
		const FSolverVec3 Difference = P1 - P;

		// Normal repulsion with some stiction
		const TTriangle<FSolverReal> Triangle(P2, P3, P4);
		const FSolverVec3 Normal = FlipNormal[Index] ? -Triangle.GetNormal() : Triangle.GetNormal();

		const FSolverReal NormalDifference = Difference.Dot(Normal);
		if (NormalDifference > Height)
		{
			continue;
		}

		const FSolverReal NormalDelta = Height - NormalDifference;

		const FSolverReal ConstraintFriction = GetConstraintFrictionCoefficient(Index);
		const FSolverVec3 Force = ProximityStiffness * NormalDelta * Normal;
		const FSolverMatrix33 DfDx = -ProximityStiffness * (((FSolverReal)1. - ConstraintFriction) * FSolverMatrix33::OuterProduct(Normal, Normal) + FSolverMatrix33(ConstraintFriction, ConstraintFriction, ConstraintFriction));

		if (Particles.InvM(Index1) > (FSolverReal)0.)
		{
			LinearSystem.AddForce(Particles, Force, Index1, Dt);
			LinearSystem.AddSymmetricForceDerivative(Particles, &DfDx, nullptr, Index1, Index1, Dt);
		}
		if (Particles.InvM(Index2) > (FSolverReal)0.)
		{
			LinearSystem.AddForce(Particles, -Barys[Index][0] * Force, Index2, Dt);
			FSolverMatrix33 DfDxScaled = -Barys[Index][0] * DfDx;
			LinearSystem.AddSymmetricForceDerivative(Particles, &DfDxScaled, nullptr, Index1, Index2, Dt);
			DfDxScaled *= -Barys[Index][0];
			LinearSystem.AddSymmetricForceDerivative(Particles, &DfDxScaled, nullptr, Index2, Index2, Dt);
		}
		if (Particles.InvM(Index3) > (FSolverReal)0.)
		{
			LinearSystem.AddForce(Particles, -Barys[Index][1] * Force, Index3, Dt);
			FSolverMatrix33 DfDxScaled = -Barys[Index][1] * DfDx;
			LinearSystem.AddSymmetricForceDerivative(Particles, &DfDxScaled, nullptr, Index1, Index3, Dt);
			DfDxScaled *= -Barys[Index][1];
			LinearSystem.AddSymmetricForceDerivative(Particles, &DfDxScaled, nullptr, Index3, Index3, Dt);
		}
		if (Particles.InvM(Index4) > (FSolverReal)0.)
		{
			LinearSystem.AddForce(Particles, -Barys[Index][2] * Force, Index4, Dt);
			FSolverMatrix33 DfDxScaled = -Barys[Index][2] * DfDx;
			LinearSystem.AddSymmetricForceDerivative(Particles, &DfDxScaled, nullptr, Index1, Index4, Dt);
			DfDxScaled *= -Barys[Index][2];
			LinearSystem.AddSymmetricForceDerivative(Particles, &DfDxScaled, nullptr, Index4, Index4, Dt);
		}
	}
}


template<typename SolverParticlesOrRange>
void FPBDCollisionSpringConstraintsBase::Apply(SolverParticlesOrRange& InParticles, const FSolverReal Dt) const
{
	ApplyDynamicConstraints(InParticles, Dt);
	if constexpr (std::is_same<SolverParticlesOrRange, FSolverParticlesRange>::value)
	{
		KinematicCollisions.Apply(InParticles, Dt);
	}
}
template CHAOS_API void FPBDCollisionSpringConstraintsBase::Apply(FSolverParticles& Particles, const FSolverReal Dt) const;
template CHAOS_API void FPBDCollisionSpringConstraintsBase::Apply(FSolverParticlesRange& Particles, const FSolverReal Dt) const;

template<typename SolverParticlesOrRange>
void FPBDCollisionSpringConstraintsBase::ApplyDynamicConstraints(SolverParticlesOrRange& Particles, const FSolverReal Dt) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ChaosPBDCollisionSpring_ApplyDynamic);

	check(Constraints.Num() == Barys.Num());
	check(Constraints.Num() == FlipNormal.Num());

	const TVec4<int32>* const ConstraintsData = Constraints.GetData();
	const FSolverVec3* const BarysData = Barys.GetData();
	const bool* const FlipNormalData = FlipNormal.GetData();

	FPAndInvM* const PAndInvM = Particles.GetPAndInvM().GetData();
	const FSolverVec3* const X = Particles.XArray().GetData();

	for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
	{
		const TVec4<int32>& Constraint = ConstraintsData[ConstraintIndex];
		const int32 Index1 = Constraint[0];
		const int32 Index2 = Constraint[1];
		const int32 Index3 = Constraint[2];
		const int32 Index4 = Constraint[3];

		const FSolverReal TrianglePointInvM =
			PAndInvM[Index2].InvM * BarysData[ConstraintIndex][0] +
			PAndInvM[Index3].InvM * BarysData[ConstraintIndex][1] +
			PAndInvM[Index4].InvM * BarysData[ConstraintIndex][2];

		const FSolverReal CombinedMass = PAndInvM[Index1].InvM + TrianglePointInvM;
		if (CombinedMass <= (FSolverReal)1e-7)
		{
			continue;
		}

		FSolverVec3& P1 = PAndInvM[Index1].P;
		FSolverVec3& P2 = PAndInvM[Index2].P;
		FSolverVec3& P3 = PAndInvM[Index3].P;
		FSolverVec3& P4 = PAndInvM[Index4].P;

		const FSolverReal Height = GetConstraintThickness(ConstraintIndex);
		const TTriangle<FSolverReal> Triangle(P2, P3, P4);
		const FSolverVec3 Normal = FlipNormalData[ConstraintIndex] ? -Triangle.GetNormal() : Triangle.GetNormal();
		const FSolverVec3 P = BarysData[ConstraintIndex][0] * P2 + BarysData[ConstraintIndex][1] * P3 + BarysData[ConstraintIndex][2] * P4;
		const FSolverVec3 Difference = P1 - P;
		const FSolverReal NormalDifference = Difference.Dot(Normal);

		// Normal repulsion with friction
		if (NormalDifference > Height)
		{
			continue;
		}

		const FSolverReal ConstraintFriction = GetConstraintFrictionCoefficient(ConstraintIndex);

		const FSolverReal NormalDelta = Height - NormalDifference;
		const FSolverVec3 RepulsionDelta = Stiffness * NormalDelta * Normal / CombinedMass; 
		FSolverVec3 FrictionDelta((FSolverReal)0.);
		if (ConstraintFriction > 0)
		{
			const FSolverVec3& X1 = X[Index1];
			const FSolverVec3 XP = BarysData[ConstraintIndex][0] * X[Index2] + BarysData[ConstraintIndex][1] * X[Index3] + BarysData[ConstraintIndex][2] * X[Index4];
			const FSolverVec3 RelativeDisplacement = (P1 - X1) - (P - XP) + (PAndInvM[Index1].InvM - TrianglePointInvM) * RepulsionDelta;
			const FSolverVec3 RelativeDisplacementTangent = RelativeDisplacement - RelativeDisplacement.Dot(Normal) * Normal;
			const FSolverReal RelativeDisplacementTangentLength = RelativeDisplacementTangent.Length();
			const FSolverReal PositionCorrection = FMath::Min(NormalDelta * ConstraintFriction, RelativeDisplacementTangentLength);
			const FSolverReal CorrectionRatio = RelativeDisplacementTangentLength < UE_SMALL_NUMBER ? 0.f : PositionCorrection / RelativeDisplacementTangentLength;
			FrictionDelta = -CorrectionRatio * RelativeDisplacementTangent / CombinedMass;
		}
		const FSolverVec3 Delta = RepulsionDelta + FrictionDelta;

		P1 += PAndInvM[Index1].InvM * Delta;
		P2 -= PAndInvM[Index2].InvM * BarysData[ConstraintIndex][0] * Delta;
		P3 -= PAndInvM[Index3].InvM * BarysData[ConstraintIndex][1] * Delta;
		P4 -= PAndInvM[Index4].InvM * BarysData[ConstraintIndex][2] * Delta;
	}
}

}  // End namespace Chaos::Softs

#endif
