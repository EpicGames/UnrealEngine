// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/PBDConstraintBaseData.h"

namespace Chaos
{
	FConstraintBase::FConstraintBase(EConstraintType InType)
		: Type(InType)
		, Proxy(nullptr)
	{
	}

	bool FConstraintBase::IsValid() const
	{
		return Proxy != nullptr;
	}

	void FConstraintBase::SetProxy(IPhysicsProxyBase* InProxy)
	{
		Proxy = InProxy;
		if (Proxy)
		{
			if (FPhysicsSolverBase* PhysicsSolverBase = Proxy->GetSolver<FPhysicsSolverBase>())
			{
				PhysicsSolverBase->AddDirtyProxy(Proxy);
			}
		}
	}

} // Chaos

