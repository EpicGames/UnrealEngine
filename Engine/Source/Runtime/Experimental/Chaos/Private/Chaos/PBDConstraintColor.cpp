// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/PBDConstraintColor.h"

//
// TO BE REMOVED
//