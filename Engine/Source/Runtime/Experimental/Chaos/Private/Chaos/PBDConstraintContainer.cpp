// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/PBDConstraintContainer.h"
#include "Chaos/Evolution/SolverConstraintContainer.h"

namespace Chaos
{
	FPBDConstraintContainer::FPBDConstraintContainer(FConstraintHandleTypeID InConstraintHandleType)
		: ConstraintHandleType(InConstraintHandleType)
		, ContainerId(INDEX_NONE)
	{
	}

	FPBDConstraintContainer::~FPBDConstraintContainer()
	{
	}

	void FPBDConstraintContainer::OnDisableParticle(FGeometryParticleHandle* DisabledParticle)
	{
		for (FConstraintHandle* ConstraintHandle : DisabledParticle->ParticleConstraints())
		{
			if ((ConstraintHandle->GetContainerId() == ContainerId) && ConstraintHandle->IsEnabled())
			{
				ConstraintHandle->SetEnabled(false);
			}
		}
	}

	void FPBDConstraintContainer::OnEnableParticle(FGeometryParticleHandle* EnabledParticle)
	{
		for (FConstraintHandle* ConstraintHandle : EnabledParticle->ParticleConstraints())
		{
			if ((ConstraintHandle->GetContainerId() == ContainerId) && !ConstraintHandle->IsEnabled())
			{
				ConstraintHandle->SetEnabled(true);
			}
		}
	}

}
