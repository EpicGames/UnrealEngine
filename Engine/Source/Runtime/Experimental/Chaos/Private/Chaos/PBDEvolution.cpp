// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/PBDEvolution.h"

#include "Chaos/Framework/Parallel.h"
#include "Chaos/PBDTriangleMeshIntersections.h"
#include "Chaos/PerParticleDampVelocity.h"
#include "Chaos/PerParticleGravity.h"
#include "Chaos/PerParticlePBDCCDCollisionConstraint.h"
#include "Chaos/PerParticlePBDCollisionConstraint.h"
#include "Chaos/VelocityField.h"
#include "ChaosStats.h"
#include "HAL/IConsoleManager.h"
#if INTEL_ISPC
#include "PBDEvolution.ispc.generated.h"
#endif

DECLARE_CYCLE_STAT(TEXT("Chaos PBD Advance Time"), STAT_ChaosPBDVAdvanceTime, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos PBD Velocity Damping State Update"), STAT_ChaosPBDVelocityDampUpdateState, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos PBD Velocity Field Update Forces"), STAT_ChaosPBDVelocityFieldUpdateForces, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos PBD Velocity Damping"), STAT_ChaosPBDVelocityDampUpdate, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos PBD Pre Iteration Updates"), STAT_ChaosPBDPreIterationUpdates, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos PBD Integrate"), STAT_ChaosClothSolverIntegrate, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos PBD Iteration Loop"), STAT_ChaosPBDIterationLoop, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos PBD Post Iteration Updates"), STAT_ChaosPBDPostIterationUpdates, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos PBD Constraint Rules"), STAT_ChaosPBDConstraintRule, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos PBD Post Collision Constraint Rules"), STAT_ChaosPBDPostCollisionConstraintRule, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos PBD Constraint Postprocessings"), STAT_ChaosPBDConstraintPostprocessings, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos PBD Self Collision"), STAT_ChaosPBDSelfCollisionRule, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos PBD Collision Rule"), STAT_ChaosPBDCollisionRule, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos PBD Collider Friction"), STAT_ChaosPBDCollisionRuleFriction, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos PBD Collider Kinematic Update"), STAT_ChaosPBDCollisionKinematicUpdate, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos PBD Clear Collided Array"), STAT_ChaosPBDClearCollidedArray, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos XPBD Constraints Init"), STAT_ChaosXPBDConstraintsInit, STATGROUP_Chaos);

TAutoConsoleVariable<bool> CVarChaosPBDEvolutionUseNestedParallelFor(TEXT("p.Chaos.PBDEvolution.UseNestedParallelFor"), true, TEXT(""), ECVF_Cheat);
TAutoConsoleVariable<bool> CVarChaosPBDEvolutionFastPositionBasedFriction(TEXT("p.Chaos.PBDEvolution.FastPositionBasedFriction"), true, TEXT(""), ECVF_Cheat);
TAutoConsoleVariable<bool> CVarChaosPBDEvolutionUseSmoothTimeStep(TEXT("p.Chaos.PBDEvolution.UseSmoothTimeStep"), true, TEXT(""), ECVF_Cheat);
TAutoConsoleVariable<int32> CVarChaosPBDEvolutionMinParallelBatchSize(TEXT("p.Chaos.PBDEvolution.MinParallelBatchSize"), 300, TEXT(""), ECVF_Cheat);
TAutoConsoleVariable<bool> CVarChaosPBDEvolutionWriteCCDContacts(TEXT("p.Chaos.PBDEvolution.WriteCCDContacts"), false, TEXT("Write CCD collision contacts and normals potentially causing the CCD collision threads to lock, allowing for debugging of these contacts."), ECVF_Cheat);
TAutoConsoleVariable<bool> CVarChaosPBDEvolutionParallelIntegrate(TEXT("p.Chaos.PBDEvolution.ParalleIntegrate"), false, TEXT("Run the integration step in parallel for."), ECVF_Cheat);

#if INTEL_ISPC && !UE_BUILD_SHIPPING
bool bChaos_PostIterationUpdates_ISPC_Enabled = CHAOS_POST_ITERATION_UPDATES_ISPC_ENABLED;
FAutoConsoleVariableRef CVarChaosPostIterationUpdatesISPCEnabled(TEXT("p.Chaos.PostIterationUpdates.ISPC"), bChaos_PostIterationUpdates_ISPC_Enabled, TEXT("Whether to use ISPC optimizations in PBD Post iteration updates"));

static_assert(sizeof(ispc::FVector3f) == sizeof(Chaos::Softs::FSolverVec3), "sizeof(ispc::FVector3f) != sizeof(Chaos::Softs::FSolverVec3");
#endif

namespace Chaos::Softs {

void FPBDEvolution::AddGroups(int32 NumGroups)
{
	// Add elements
	const uint32 Offset = TArrayCollection::Size();
	TArrayCollection::AddElementsHelper(NumGroups);

	// Set defaults
	for (uint32 GroupId = Offset; GroupId < TArrayCollection::Size(); ++GroupId)
	{
		MGroupGravityAccelerations[GroupId] = MGravity;
		MGroupCollisionThicknesses[GroupId] = MCollisionThickness;
		MGroupCoefficientOfFrictions[GroupId] = MCoefficientOfFriction;
		MGroupDampings[GroupId] = MDamping;
		MGroupLocalDampings[GroupId] = MLocalDamping;
		MGroupUseCCDs[GroupId]  = false;
	}
}

void FPBDEvolution::ResetGroups()
{
	TArrayCollection::ResizeHelper(0);
	AddGroups(1);  // Add default group
}

FPBDEvolution::FPBDEvolution(
	FSolverParticles&& InParticles,
	FSolverCollisionParticles&& InGeometryParticles,
	TArray<TVec3<int32>>&& CollisionTriangles,
	int32 NumIterations, FSolverReal CollisionThickness,
	FSolverReal SelfCollisionThickness,
	FSolverReal CoefficientOfFriction,
	FSolverReal Damping,
	FSolverReal LocalDamping,
	bool bDoQuasistatics, 
	bool InbUsePerParticleDamping)
	: MParticles(MoveTemp(InParticles))
	, MParticlesActiveView(MParticles)
	, MCollisionParticles(MoveTemp(InGeometryParticles))
	, MCollisionParticlesActiveView(MCollisionParticles)
	, MConstraintInitsActiveView(MConstraintInits)
	, MConstraintRulesActiveView(MConstraintRules)
	, MPostCollisionConstraintRulesActiveView(MPostCollisionConstraintRules)
	, MConstraintPostprocessingsActiveView(MConstraintPostprocessings)
	, MNumIterations(NumIterations)
	, MGravity(FSolverVec3((FSolverReal)0., (FSolverReal)0., (FSolverReal)-980.665))
	, MCollisionThickness(CollisionThickness)
	, MCoefficientOfFriction(CoefficientOfFriction)
	, MDamping(Damping)
	, MLocalDamping(LocalDamping)
	, MTime(0)
	, bDoQuasistatics(bDoQuasistatics)
	, bUsePerParticleDamping(InbUsePerParticleDamping)
{
	// Add group arrays
	TArrayCollection::AddArray(&MGroupGravityAccelerations);
	TArrayCollection::AddArray(&MGroupVelocityAndPressureFields);
	TArrayCollection::AddArray(&MGroupForceRules);
	TArrayCollection::AddArray(&MGroupCollisionThicknesses);
	TArrayCollection::AddArray(&MGroupCoefficientOfFrictions);
	TArrayCollection::AddArray(&MGroupDampings);
	TArrayCollection::AddArray(&MGroupLocalDampings);
	TArrayCollection::AddArray(&MGroupUseCCDs);
	AddGroups(1);  // Add default group

	// Add particle arrays
	MParticles.AddArray(&MParticleGroupIds);
	if (bUsePerParticleDamping)
	{
		MParticles.AddArray(&MParticleDampings);
	}
	MCollisionParticles.AddArray(&MCollisionTransforms);
	MCollisionParticles.AddArray(&MCollided);
	MCollisionParticles.AddArray(&MCollisionParticleGroupIds);
}

void FPBDEvolution::ResetParticles()
{
	// Reset particles
	MParticles.Resize(0);
	MParticlesActiveView.Reset();

	// Reset particle groups
	ResetGroups();
}

int32 FPBDEvolution::AddParticleRange(int32 NumParticles, uint32 GroupId, bool bActivate)
{
	if (NumParticles)
	{
		const int32 Offset = (int32)MParticles.Size();

		MParticles.AddParticles(NumParticles);

		// Initialize the new particles' group ids
		for (int32 i = Offset; i < (int32)MParticles.Size(); ++i)
		{
			MParticleGroupIds[i] = GroupId;
		}

		if (bUsePerParticleDamping)
		{
			for (int32 i = Offset; i < (int32)MParticles.Size(); ++i)
			{
				MParticleDampings[i] = (FSolverReal)0.;
			}
		}

		// Resize the group parameter arrays
		const uint32 GroupSize = TArrayCollection::Size();
		if (GroupId >= GroupSize)
		{
			AddGroups(GroupId + 1 - GroupSize);
		}

		// Add range
		MParticlesActiveView.AddRange(NumParticles, bActivate);

		return Offset;
	}
	return INDEX_NONE;
}

void FPBDEvolution::ResetCollisionParticles(int32 NumParticles)
{
	MCollisionParticles.Resize(NumParticles);
	MCollisionParticlesActiveView.Reset(NumParticles);
}

int32 FPBDEvolution::AddCollisionParticleRange(int32 NumParticles, uint32 GroupId, bool bActivate)
{
	if (NumParticles)
	{
		const int32 RangeOffset = (int32)MCollisionParticles.Size();

		MCollisionParticles.AddParticles(NumParticles);

		// Initialize the new particles' group ids
		for (int32 i = RangeOffset; i < (int32)MCollisionParticles.Size(); ++i)
		{
			MCollisionParticleGroupIds[i] = GroupId;
		}

		// Add range
		MCollisionParticlesActiveView.AddRange(NumParticles, bActivate);
	
		return RangeOffset;
	}
	return INDEX_NONE;
}

int32 FPBDEvolution::AddCollisionParticle(uint32 GroupId, bool bActivate)
{
	FParticleVievToken Token = { INDEX_NONE,INDEX_NONE };
	if (!RemovedCollisionIndices.Num())
	{
		Token = { (int32)MCollisionParticles.Size(),INDEX_NONE };

		MCollisionParticles.AddParticles(1);
	}
	else
	{
		Token = RemovedCollisionIndices.Pop(); 
	}

	if (0 <= Token.ParticleIndex && Token.ParticleIndex < (int32)MCollisionParticles.Size())
	{
		MCollisionParticleGroupIds[Token.ParticleIndex] = GroupId;

		if (Token.ViewIndex == INDEX_NONE)
		{
			// Add range
			MCollisionParticlesActiveView.AddRange(1, bActivate);
		}
		else
		{
			// Update Range
			MCollisionParticlesActiveView.ActivateRange(Token.ViewIndex, bActivate);
		}
	}

	return Token.ParticleIndex;
}

void FPBDEvolution::RemoveCollisionParticle(int32 CollisionParticleIndex, int32 CollisionParticleViewIndex)
{
	if (0 <= CollisionParticleIndex && CollisionParticleIndex < (int32)MCollisionParticles.Size())
	{
		CollisionParticleGroupIds()[CollisionParticleIndex] = CollisionParticleIndex;
		MCollisionParticlesActiveView.ActivateRange(CollisionParticleViewIndex, false);
		RemovedCollisionIndices.Add({ CollisionParticleIndex ,CollisionParticleViewIndex });
	}
	else
	{
		ensure(false);
	}
}


int32 FPBDEvolution::AddConstraintInitRange(int32 NumConstraints, bool bActivate)
{
	// Add new constraint init functions
	MConstraintInits.AddDefaulted(NumConstraints);

	// Add range
	return MConstraintInitsActiveView.AddRange(NumConstraints, bActivate);
}

int32 FPBDEvolution::AddConstraintRuleRange(int32 NumConstraints, bool bActivate)
{
	// Add new constraint rule functions
	MConstraintRules.AddDefaulted(NumConstraints);

	// Add range
	return MConstraintRulesActiveView.AddRange(NumConstraints, bActivate);
}

int32 FPBDEvolution::AddPostCollisionConstraintRuleRange(int32 NumConstraints, bool bActivate)
{
	// Add new constraint rule functions
	MPostCollisionConstraintRules.AddDefaulted(NumConstraints);

	// Add range
	return MPostCollisionConstraintRulesActiveView.AddRange(NumConstraints, bActivate);
}

int32 FPBDEvolution::AddConstraintPostprocessingsRange(int32 NumConstraints, bool bActivate)
{
	// Add new constraint rule functions
	MConstraintPostprocessings.AddDefaulted(NumConstraints);

	// Add range
	return MConstraintPostprocessingsActiveView.AddRange(NumConstraints, bActivate);
}

template<bool bForceRule, bool bVelocityField, bool bDampVelocityRule>
void FPBDEvolution::PreIterationUpdate(
	const FSolverReal Dt,
	const int32 Offset,
	const int32 Range,
	const int32 MinParallelBatchSize)
{
	const uint32 ParticleGroupId = MParticleGroupIds[Offset];
	const TFunction<void(FSolverParticles&, const FSolverReal, const int32)>& ForceRule = MGroupForceRules[ParticleGroupId];
	const FSolverVec3& Gravity = MGroupGravityAccelerations[ParticleGroupId];
	FVelocityAndPressureField& VelocityAndPressureField = MGroupVelocityAndPressureFields[ParticleGroupId];

	if (bVelocityField)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ChaosPBDVelocityFieldUpdateForces);
		SCOPE_CYCLE_COUNTER(STAT_ChaosPBDVelocityFieldUpdateForces);
		VelocityAndPressureField.UpdateForces(MParticles, Dt);  // Update force per surface element
	}

	FPerParticleDampVelocity DampVelocityRule(MGroupLocalDampings[ParticleGroupId]);
	if (bDampVelocityRule)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ChaosPBDVelocityDampUpdateState);
		SCOPE_CYCLE_COUNTER(STAT_ChaosPBDVelocityDampUpdateState);
		DampVelocityRule.UpdatePositionBasedState(MParticles, Offset, Range);
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ChaosClothSolverIntegrate);
		SCOPE_CYCLE_COUNTER(STAT_ChaosClothSolverIntegrate);

		constexpr FSolverReal DampingFrequency = (FSolverReal)60.;  // The damping value is the percentage of velocity removed per frame when running at 60Hz
		FSolverReal Damping = FMath::Clamp(MGroupDampings[ParticleGroupId], (FSolverReal)0., (FSolverReal)1.);
		FSolverReal DampingPowDt;
		FSolverReal DampingIntegrated;
		if (Damping > (FSolverReal)1. - (FSolverReal)UE_KINDA_SMALL_NUMBER)
		{
			DampingIntegrated = DampingPowDt = (FSolverReal)0.;
		}
		else if (Damping > (FSolverReal)UE_SMALL_NUMBER)
		{
			const FSolverReal LogValueByFrequency = FMath::Loge((FSolverReal)1. - Damping) * DampingFrequency;

			DampingPowDt = FMath::Exp(LogValueByFrequency * Dt);  // DampingPowDt = FMath::Pow(OneMinusDamping, Dt * DampingFrequency);
			DampingIntegrated = (DampingPowDt - (FSolverReal)1.) / LogValueByFrequency;
		}
		else
		{
			DampingPowDt = (FSolverReal)1.;
			DampingIntegrated = Dt;
		}

		if (bDoQuasistatics)
		{
			const int32 RangeSize = Range - Offset;
			PhysicsParallelFor(RangeSize,
				[this, &Offset, &ForceRule, &Gravity, &VelocityAndPressureField, &DampVelocityRule, DampingPowDt, DampingIntegrated, Dt](int32 i)
				{
					const int32 Index = Offset + i;
					if (MParticles.InvM(Index) != (FSolverReal)0.)  // Process dynamic particles
					{
						MParticles.SetP(Index, MParticles.GetX(Index));
					}
					else  // Process kinematic particles
					{
						MKinematicUpdate(MParticles, Dt, MTime, Index);
					}
				}, RangeSize < MinParallelBatchSize);
		} 
		else if (bUsePerParticleDamping)
		{
			const int32 RangeSize = Range - Offset;
			PhysicsParallelFor(RangeSize,
				[this, &Offset, &ForceRule, &Gravity, &VelocityAndPressureField, &DampVelocityRule, &DampingFrequency, &ParticleGroupId, DampingPowDt, DampingIntegrated, Dt](int32 i)
				{
					const int32 Index = Offset + i;

					const FSolverReal ParticleDamping = FMath::Clamp(MParticleDampings[Index] * MGroupDampings[ParticleGroupId], (FSolverReal)0., (FSolverReal)1.);
					FSolverReal ParticleDampingPowDt;
					FSolverReal ParticleDampingIntegrated;
					if (ParticleDamping > (FSolverReal)1. - (FSolverReal)UE_KINDA_SMALL_NUMBER)
					{
						ParticleDampingIntegrated = ParticleDampingPowDt = (FSolverReal)0.;
					}
					else if (ParticleDamping > (FSolverReal)UE_SMALL_NUMBER)
					{
						const FSolverReal ParticleLogValueByFrequency = FMath::Loge((FSolverReal)1. - ParticleDamping) * DampingFrequency;

						ParticleDampingPowDt = FMath::Exp(ParticleLogValueByFrequency * Dt);  // DampingPowDt = FMath::Pow(OneMinusDamping, Dt * DampingFrequency);
						ParticleDampingIntegrated = (ParticleDampingPowDt - (FSolverReal)1.) / ParticleLogValueByFrequency;
					}
					else
					{
						ParticleDampingPowDt = (FSolverReal)1.;
						ParticleDampingIntegrated = Dt;
					}
					
					if (MParticles.InvM(Index) != (FSolverReal)0.)  // Process dynamic particles
					{
						// Init forces with GravityForces
						MParticles.Acceleration(Index) = Gravity;

						// Force Rule
						if (bForceRule)
						{
							ForceRule(MParticles, Dt, Index); // F += M * A
						}

						// Velocity Field
						if (bVelocityField)
						{
							VelocityAndPressureField.Apply(MParticles, Dt, Index);
						}

						// Euler Step Velocity
						MParticles.V(Index) += MParticles.Acceleration(Index) * Dt;

						// Damp Velocity Rule
						if (bDampVelocityRule)
						{
							DampVelocityRule.ApplyFast(MParticles, Dt, Index);
						}

						// Euler Step with point damping integration
						MParticles.SetP(Index, MParticles.GetX(Index) + MParticles.GetV(Index) * ParticleDampingIntegrated);

						MParticles.V(Index) *= ParticleDampingPowDt;
					}
					else  // Process kinematic particles
					{
						MKinematicUpdate(MParticles, Dt, MTime, Index);
					}
				}, RangeSize < MinParallelBatchSize);
			
		}
		else
		{
			const int32 RangeSize = Range - Offset;
			PhysicsParallelFor(RangeSize,
				[this, &Offset, &ForceRule, &Gravity, &VelocityAndPressureField, &DampVelocityRule, &DampingFrequency, &ParticleGroupId, DampingPowDt, DampingIntegrated, Dt](int32 i)
				{
					const int32 Index = Offset + i;

					if (MParticles.InvM(Index) != (FSolverReal)0.)  // Process dynamic particles
					{
						// Init forces with GravityForces
						MParticles.Acceleration(Index) = Gravity;

						// Force Rule
						if (bForceRule)
						{
							ForceRule(MParticles, Dt, Index); // F += M * A
						}

						// Velocity Field
						if (bVelocityField)
						{
							VelocityAndPressureField.Apply(MParticles, Dt, Index);
						}

						// Euler Step Velocity
						MParticles.V(Index) += MParticles.Acceleration(Index) * Dt;

						// Damp Velocity Rule
						if (bDampVelocityRule)
						{
							DampVelocityRule.ApplyFast(MParticles, Dt, Index);
						}

						// Euler Step with point damping integration
						MParticles.SetP(Index, MParticles.GetX(Index) + MParticles.GetV(Index) * DampingIntegrated);

						MParticles.V(Index) *= DampingPowDt;
					}
					else  // Process kinematic particles
					{
						MKinematicUpdate(MParticles, Dt, MTime, Index);
					}
				}, RangeSize < MinParallelBatchSize);
		}
	}
}

void FPBDEvolution::AdvanceOneTimeStep(const FSolverReal Dt)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPBDEvolution_AdvanceOneTimeStep);
	SCOPE_CYCLE_COUNTER(STAT_ChaosPBDVAdvanceTime);

	// Advance time
	MTime += Dt;

	// Don't bother with threaded execution if we don't have enough work to make it worth while.
	const bool bUseSingleThreadedRange = !CVarChaosPBDEvolutionUseNestedParallelFor.GetValueOnAnyThread();
	const int32 MinParallelBatchSize = !CVarChaosPBDEvolutionParallelIntegrate.GetValueOnAnyThread() ?
		TNumericLimits<int32>::Max() :  // Disable
		CVarChaosPBDEvolutionMinParallelBatchSize.GetValueOnAnyThread(); // TODO: 1000 is a guess, tune this!
	const bool bWriteCCDContacts = CVarChaosPBDEvolutionWriteCCDContacts.GetValueOnAnyThread();

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ChaosPBDPreIterationUpdates);
		SCOPE_CYCLE_COUNTER(STAT_ChaosPBDPreIterationUpdates);

		MParticlesActiveView.RangeFor(
			[this, Dt, MinParallelBatchSize](FSolverParticles& Particles, int32 Offset, int32 Range)
			{
				const uint32 ParticleGroupId = MParticleGroupIds[Offset];

				if (MGroupVelocityAndPressureFields[ParticleGroupId].IsActive())
				{
					if (MGroupLocalDampings[ParticleGroupId] > (FSolverReal)0.)
					{
						if (MGroupForceRules[ParticleGroupId])  // VeloctiyFields, Damping, Forces  // Damping?????
						{
							PreIterationUpdate</*bForceRule =*/ true, /*bVelocityField =*/ true, /*bDampVelocityRule =*/ true>(Dt, Offset, Range, MinParallelBatchSize);
						}
						else  // VeloctiyFields, Damping
						{
							PreIterationUpdate</*bForceRule =*/ false, /*bVelocityField =*/ true, /*bDampVelocityRule =*/ true>(Dt, Offset, Range, MinParallelBatchSize);
						}
					}
					else  // No Damping
					{
						if (MGroupForceRules[ParticleGroupId])  // VeloctiyFields, Forces
						{
							PreIterationUpdate</*bForceRule =*/ true, /*bVelocityField =*/ true, /*bDampVelocityRule =*/ false>(Dt, Offset, Range, MinParallelBatchSize);
						}
						else  // VeloctiyFields
						{
							PreIterationUpdate</*bForceRule =*/ false, /*bVelocityField =*/ true, /*bDampVelocityRule =*/ false>(Dt, Offset, Range, MinParallelBatchSize);
						}
					}
				}
				else   // No Velocity Fields
				{
					if (MGroupLocalDampings[ParticleGroupId] > (FSolverReal)0.)
					{
						if (MGroupForceRules[ParticleGroupId])  // VeloctiyFields, Damping, Forces
						{
							PreIterationUpdate</*bForceRule =*/ true, /*bVelocityField =*/ false, /*bDampVelocityRule =*/ true>(Dt, Offset, Range, MinParallelBatchSize);
						}
						else  // VeloctiyFields, Damping
						{
							PreIterationUpdate</*bForceRule =*/ false, /*bVelocityField =*/ false, /*bDampVelocityRule =*/ true>(Dt, Offset, Range, MinParallelBatchSize);
						}
					}
					else  // No Damping
					{
						if (MGroupForceRules[ParticleGroupId])  // VeloctiyFields, Forces
						{
							PreIterationUpdate</*bForceRule =*/ true, /*bVelocityField =*/ false, /*bDampVelocityRule =*/ false>(Dt, Offset, Range, MinParallelBatchSize);
						}
						else  // VeloctiyFields
						{
							PreIterationUpdate</*bForceRule =*/ false, /*bVelocityField =*/ false, /*bDampVelocityRule =*/ false>(Dt, Offset, Range, MinParallelBatchSize);
						}
					}
				}
			}, bUseSingleThreadedRange);
	}

	// Collision update
	{
		if (MCollisionKinematicUpdate)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ChaosPBDCollisionKinematicUpdate);
			SCOPE_CYCLE_COUNTER(STAT_ChaosPBDCollisionKinematicUpdate);

			MCollisionParticlesActiveView.SequentialFor(
				[this, Dt](FSolverCollisionParticles& CollisionParticles, int32 Index)
				{
					// Store active collision particle frames prior to the kinematic update for CCD collisions
					MCollisionTransforms[Index] = FSolverRigidTransform3(CollisionParticles.GetX(Index), CollisionParticles.GetR(Index));

					// Update collision transform and velocity
					MCollisionKinematicUpdate(CollisionParticles, Dt, MTime, Index); // This expects Sequential update.
				});
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ChaosPBDClearCollidedArray);
			SCOPE_CYCLE_COUNTER(STAT_ChaosPBDClearCollidedArray);
			memset(MCollided.GetData(), 0, MCollided.Num() * sizeof(bool));
		}
	}

	// Constraint init (clear XPBD's Lambdas, init self collisions)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ChaosXPBDConstraintsInit);
		SCOPE_CYCLE_COUNTER(STAT_ChaosXPBDConstraintsInit);
		MConstraintInitsActiveView.SequentialFor(
			[this, Dt](TArray<TFunction<void(FSolverParticles&, const FSolverReal)>>& ConstraintInits, int32 Index)
			{
				ConstraintInits[Index](MParticles, Dt);
			});
	}

	// Collision rule initializations
	MCollisionContacts.Reset();
	MCollisionNormals.Reset();
	MCollisionPhis.Reset();

	FPerParticlePBDCollisionConstraint CollisionRule(
		MCollisionParticlesActiveView,
		MCollided,
		MCollisionContacts,
		MCollisionNormals,
		MCollisionPhis,
		MParticleGroupIds,
		MCollisionParticleGroupIds,
		MGroupCollisionThicknesses,
		MGroupCoefficientOfFrictions,
		bWriteCCDContacts);

	FPerParticlePBDCCDCollisionConstraint CCDCollisionRule(
		MCollisionParticlesActiveView,
		MCollisionTransforms,
		MCollided,
		MCollisionContacts,
		MCollisionNormals,
		MParticleGroupIds,
		MCollisionParticleGroupIds,
		MGroupCollisionThicknesses,
		MGroupCoefficientOfFrictions,
		bWriteCCDContacts);

	// Iteration loop
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ChaosPBDIterationLoop);
		SCOPE_CYCLE_COUNTER(STAT_ChaosPBDIterationLoop);

		for (int32 i = 0; i < MNumIterations; ++i)
		{
			MConstraintRulesActiveView.RangeFor(
				[this, Dt](TArray<TFunction<void(FSolverParticles&, const FSolverReal)>>& ConstraintRules, int32 Offset, int32 Range)
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(ChaosPBDConstraintRule);
					SCOPE_CYCLE_COUNTER(STAT_ChaosPBDConstraintRule);
					for (int32 ConstraintIndex = Offset; ConstraintIndex < Range; ++ConstraintIndex)
					{
						ConstraintRules[ConstraintIndex](MParticles, Dt); // P +/-= ...
					}
				}, bUseSingleThreadedRange);

			{
				TRACE_CPUPROFILER_EVENT_SCOPE(ChaosPBDCollisionRule);
				SCOPE_CYCLE_COUNTER(STAT_ChaosPBDCollisionRule);
				MParticlesActiveView.RangeFor(
					[this, &CollisionRule, &CCDCollisionRule, Dt](FSolverParticles& Particles, int32 Offset, int32 Range)
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(ChaosPBDCollisionRuleP);
						const uint32 DynamicGroupId = MParticleGroupIds[Offset];  // Particle group Id, must be the same across the entire range
						const bool bUseCCD = MGroupUseCCDs[DynamicGroupId];
						if (!bUseCCD)
						{
							CollisionRule.ApplyRange(Particles, Dt, Offset, Range);
						}
						else
						{
							CCDCollisionRule.ApplyRange(Particles, Dt, Offset, Range);
						}
					}, bUseSingleThreadedRange);
			}
			MPostCollisionConstraintRulesActiveView.RangeFor(
				[this, Dt](TArray<TFunction<void(FSolverParticles&, const FSolverReal)>>& ConstraintRules, int32 Offset, int32 Range)
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(ChaosPBDPostCollisionConstraintRule);
					SCOPE_CYCLE_COUNTER(STAT_ChaosPBDPostCollisionConstraintRule);
					for (int32 ConstraintIndex = Offset; ConstraintIndex < Range; ++ConstraintIndex)
					{
						ConstraintRules[ConstraintIndex](MParticles, Dt); // P +/-= ...
					}
				}, bUseSingleThreadedRange);
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ChaosPBDConstraintPostprocessings);
			SCOPE_CYCLE_COUNTER(STAT_ChaosPBDConstraintPostprocessings);
			MConstraintPostprocessingsActiveView.RangeFor(
				[this, Dt](TArray<TFunction<void(FSolverParticles&, const FSolverReal)>>& ConstraintPostprocessings, int32 Offset, int32 Range)
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(ChaosPBDConstraintPostprocessings);
					SCOPE_CYCLE_COUNTER(STAT_ChaosPBDConstraintPostprocessings);
					for (int32 ConstraintIndex = Offset; ConstraintIndex < Range; ++ConstraintIndex)
					{
						ConstraintPostprocessings[ConstraintIndex](MParticles, Dt); // P +/-= ...
					}
				}, bUseSingleThreadedRange);
		}

		{
			SCOPE_CYCLE_COUNTER(STAT_ChaosPBDPostIterationUpdates);

			// Particle update, V = (P - X) / Dt; X = P;
#if INTEL_ISPC
			if (bChaos_PostIterationUpdates_ISPC_Enabled)
			{
				MParticlesActiveView.RangeFor(
					[Dt](FSolverParticles& Particles, int32 Offset, int32 Range)
					{
						ispc::PostIterationUpdates(
							(ispc::FVector3f*)Particles.GetV().GetData(),
							(ispc::FVector3f*)Particles.XArray().GetData(),
							(const ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
							Dt,
							Offset,
							Range);
					});
			}
			else
#endif
			{
				MParticlesActiveView.ParallelFor(
					[Dt](FSolverParticles& Particles, int32 Index)
					{
						Particles.SetV(Index, (Particles.GetP(Index) - Particles.GetX(Index)) / Dt);
						Particles.SetX(Index, Particles.GetP(Index));
					}, MinParallelBatchSize);
			}
		}
	}

	// The following is not currently been used by the cloth solver implementation at the moment
	if (!CVarChaosPBDEvolutionFastPositionBasedFriction.GetValueOnAnyThread() && MCoefficientOfFriction > 0)
	{
		SCOPE_CYCLE_COUNTER(STAT_ChaosPBDCollisionRuleFriction);
		MParticlesActiveView.ParallelFor(
			[&CollisionRule, Dt](FSolverParticles& Particles, int32 Index)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(ChaosPBDCollisionRuleFriction);
				CollisionRule.ApplyFriction(Particles, Dt, Index);
			}, bUseSingleThreadedRange, MinParallelBatchSize);
	}
}

}  // End namespace Chaos::Softs
