// Copyright Epic Games, Inc. All Rights Reserved.

#define EXPLICIT_VECTOR4 1

#include "Math/Vector.isph"

export void PostIterationUpdates(uniform FVector3f ParticlesV[],
								uniform FVector3f ParticlesX[],
								const uniform FVector4f ParticlesPandInvM[],
								const uniform float Dt,
								const uniform int32 Offset,
								const uniform int32 Range)
{
	foreach(i = Offset ... Range)
	{
		const uniform int32 Index = extract(i, 0);

		const varying FVector4f PandInvM = VectorLoad(&ParticlesPandInvM[Index]);
		const varying FVector3f P = {{PandInvM.V[0], PandInvM.V[1], PandInvM.V[2]}};
		const varying FVector3f X = VectorLoad(&ParticlesX[Index]);

		VectorStore(&ParticlesV[Index], (P - X) / Dt);
		VectorStore(&ParticlesX[Index], P);
	}
}
