// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/PBDExtremeDeformationConstraints.h"
#include "HAL/IConsoleManager.h"

namespace Chaos::Softs
{

namespace Private
{
	bool bOverrideExtremeDeformationEdgeRatioThreshold = false;
	FAutoConsoleVariableRef CVarOverrideExtremeDeformationEdgeRatioThreshold(TEXT("p.ExtremeDeformationConstraints.OverrideExtremeDeformationEdgeRatioThreshold"), bOverrideExtremeDeformationEdgeRatioThreshold, TEXT("Override asset-based extreme deformation edge ratio threshold with GlobalExtremeDeformationEdgeRatioThreshold."));
	float GlobalExtremeDeformationEdgeRatioThreshold = FLT_MAX;
	FAutoConsoleVariableRef CVarGlobalExtremeDeformationEdgeRatioThreshold(TEXT("p.ExtremeDeformationConstraints.GlobalExtremeDeformationEdgeRatioThreshold"), GlobalExtremeDeformationEdgeRatioThreshold, TEXT("Global extreme deformation edge ratio threshold parameter to trigger position reset."));
}

FSolverReal FPBDExtremeDeformationConstraints::GetThreshold() const
{
	if (Private::bOverrideExtremeDeformationEdgeRatioThreshold)
	{
		return Private::GlobalExtremeDeformationEdgeRatioThreshold;
	}
	return ExtremeDeformationThreshold;
}

bool FPBDExtremeDeformationConstraints::IsExtremelyDeformed(TConstArrayView<Softs::FSolverVec3> Positions) const
{
	const FSolverReal Threshold = GetThreshold();
	for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
	{
		const auto& Constraint = Constraints[ConstraintIndex];
		const int32 i1 = Constraint[0];
		const int32 i2 = Constraint[1];
		const FSolverVec3& P1 = Positions[i1];
		const FSolverVec3& P2 = Positions[i2];
		const FSolverReal EdgeLengthRatio = (P1 - P2).Size() / Dists[ConstraintIndex];
		if (EdgeLengthRatio > Threshold)
		{
			return true;
		}
	}
	return false;
}

bool FPBDExtremeDeformationConstraints::IsExtremelyDeformed(TConstArrayView<Softs::FSolverVec3> Positions, const Softs::FSolverVec3* const RefencePositions) const
{
	const FSolverReal Threshold = GetThreshold();
	for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
	{
		const auto& Constraint = Constraints[ConstraintIndex];
		const int32 i1 = Constraint[0];
		const int32 i2 = Constraint[1];
		const FSolverVec3& P1 = Positions[i1];
		const FSolverVec3& P2 = Positions[i2];
		const FSolverVec3& RefP1 = RefencePositions[i1];
		const FSolverVec3& RefP2 = RefencePositions[i2];
		const FSolverReal EdgeLengthRatio = (P1 - P2).Size() / (RefP1 - RefP2).Size();
		if (EdgeLengthRatio > Threshold)
		{
			return true;
		}
	}
	return false;
}

TArray<TVec2<int32>> FPBDExtremeDeformationConstraints::GetExtremelyDeformedEdges(TConstArrayView<Softs::FSolverVec3> Positions) const
{
	const FSolverReal Threshold = GetThreshold();
	TArray<TVec2<int32>> OutEdges;
	for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
	{
		const TVec2<int32>& Constraint = Constraints[ConstraintIndex];
		const int32 i1 = Constraint[0];
		const int32 i2 = Constraint[1];
		const FSolverVec3& P1 = Positions[i1];
		const FSolverVec3& P2 = Positions[i2];
		const FSolverReal EdgeLengthRatio = (P1 - P2).Size() / Dists[ConstraintIndex];
		if (EdgeLengthRatio > Threshold)
		{
			OutEdges.Add(Constraint);
		}
	}
	return OutEdges;
}

TArray<TVec2<int32>> FPBDExtremeDeformationConstraints::GetExtremelyDeformedEdges(TConstArrayView<Softs::FSolverVec3> Positions, const Softs::FSolverVec3* const RefencePositions) const
{
	const FSolverReal Threshold = GetThreshold();
	TArray<TVec2<int32>> OutEdges;
	for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
	{
		const TVec2<int32>& Constraint = Constraints[ConstraintIndex];
		const int32 i1 = Constraint[0];
		const int32 i2 = Constraint[1];
		const FSolverVec3& P1 = Positions[i1];
		const FSolverVec3& P2 = Positions[i2];
		const FSolverVec3& RefP1 = RefencePositions[i1];
		const FSolverVec3& RefP2 = RefencePositions[i2];
		const FSolverReal EdgeLengthRatio = (P1 - P2).Size() / (RefP1 - RefP2).Size();
		if (EdgeLengthRatio > Threshold)
		{
			OutEdges.Add(Constraint);
		}
	}
	return OutEdges;
}

} // End namespace Chaos::Softs