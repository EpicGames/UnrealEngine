// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/PBDJointConstraintData.h"
#include "PBDRigidsSolver.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "Chaos/PhysicsObjectInterface.h"

namespace Chaos
{
	FJointConstraint::FJointConstraint()
		: FConstraintBase(EConstraintType::JointConstraintType)
		, KinematicEndPoint(nullptr)
	{
	}

	void FJointConstraint::SetJointSettings(const FPBDJointSettings& NewSettings)
	{
		JointSettings.Modify(/*bInvalidate=*/true, DirtyFlags, Proxy, [&NewSettings](auto& Data)
		{
			Data = NewSettings;
		});
	}

	void FJointConstraint::SetKinematicEndPoint(FSingleParticlePhysicsProxy* InDummyParticle, FPBDRigidsSolver* Solver)
	{
		ensure(KinematicEndPoint == nullptr);
		KinematicEndPoint = InDummyParticle;
		Solver->RegisterObject(KinematicEndPoint);
	}

	FSingleParticlePhysicsProxy* FJointConstraint::GetKinematicEndPoint() const
	{
		return KinematicEndPoint;
	}

	void FJointConstraint::SetLinearPositionDriveEnabled(TVector<bool,3> Enabled)
	{
		SetLinearPositionDriveXEnabled(Enabled.X);
		SetLinearPositionDriveYEnabled(Enabled.Y);
		SetLinearPositionDriveZEnabled(Enabled.Z);
	}


	void FJointConstraint::SetLinearVelocityDriveEnabled(TVector<bool,3> Enabled)
	{
		SetLinearVelocityDriveXEnabled(Enabled.X);
		SetLinearVelocityDriveYEnabled(Enabled.Y);
		SetLinearVelocityDriveZEnabled(Enabled.Z);
	}

	void FJointConstraint::ReleaseKinematicEndPoint(FPBDRigidsSolver* Solver)
	{
		if (KinematicEndPoint)
		{
			Solver->UnregisterObject(KinematicEndPoint);
			KinematicEndPoint = nullptr;
		}
	}

	void FJointConstraint::SetParticleProxies(const FProxyBasePair& InJointParticles)
	{
		JointProxies.Modify(/*bInvalidate=*/true, DirtyFlags, Proxy, [&InJointParticles](FProxyBasePairProperty& Data)
		{
			Data.ParticleProxies[0] = InJointParticles[0];
			Data.ParticleProxies[1] = InJointParticles[1];
		});

		// This should work fine since this is a legacy endpoint and was primarily only used with single particle physics proxies.
		JointBodies.Modify(/*bInvalidate=*/true, DirtyFlags, Proxy, [&InJointParticles](FPhysicsObjectPairProperty& Data)
		{
			Data.PhysicsBodies[0] = (InJointParticles[0]->GetType() == EPhysicsProxyType::SingleParticleProxy) ? static_cast<FSingleParticlePhysicsProxy*>(InJointParticles[0])->GetPhysicsObject() : nullptr;
			Data.PhysicsBodies[1] = (InJointParticles[1]->GetType() == EPhysicsProxyType::SingleParticleProxy) ? static_cast<FSingleParticlePhysicsProxy*>(InJointParticles[1])->GetPhysicsObject() : nullptr;
		});
	}

	void FJointConstraint::SetPhysicsBodies(const FPhysicsObjectPair& InBodies)
	{
		JointProxies.Modify(/*bInvalidate=*/true, DirtyFlags, Proxy, [&InBodies](FProxyBasePairProperty& Data)
		{
			Chaos::FPhysicsObject* Object1 = InBodies[0];
			Chaos::FPhysicsObject* Object2 = InBodies[1];
			Data.ParticleProxies[0] = Chaos::FPhysicsObjectInterface::GetProxy({ &Object1, 1 });
			Data.ParticleProxies[1] = Chaos::FPhysicsObjectInterface::GetProxy({ &Object2, 1 });
		});

		JointBodies.Modify(/*bInvalidate=*/true, DirtyFlags, Proxy, [&InBodies](FPhysicsObjectPairProperty& Data)
		{
			Data.PhysicsBodies[0] = InBodies[0];
			Data.PhysicsBodies[1] = InBodies[1];
		});
	}

} // Chaos
