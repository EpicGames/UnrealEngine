// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/PBDJointConstraintUtilities.h"
#include "Chaos/DenseMatrix.h"
#include "Chaos/Joint/JointConstraintsCVars.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/Utilities.h"

//PRAGMA_DISABLE_OPTIMIZATION

namespace Chaos
{
	bool bExcludeFreeJointInertiaConditioning = true;
	FAutoConsoleVariableRef CVarChaosExcludeFreeJointInertiaConditioning(TEXT("p.Chaos.ExcludeFreeJointForInertiaConditioning"), bExcludeFreeJointInertiaConditioning, TEXT("Exclude the effects from free joints when computing inertia conditioning"));

	FReal GetSwingAngleY(const FRotation3& RSwing)
	{
		return 4.0f * FMath::Atan2(RSwing.Y, 1.0f + RSwing.W);
	}

	FReal GetSwingAngleZ(const FRotation3& RSwing)
	{
		return 4.0f * FMath::Atan2(RSwing.Z, 1.0f + RSwing.W);
	}

	// Iterative find near point on ellipse. Point InP, Radii R.
	// https://www.geometrictools.com/Documentation/DistancePointEllipseEllipsoid.pdf
	FVec2 NearPointOnEllipse(const FVec2& P, const FVec2& R, const int MaxIts = 20, const FReal Tolerance = 1.e-4f)
	{
		// Map point into first quadrant
		FVec2 PAbs = P.GetAbs();

		// Check for point on minor axis
		const FReal Epsilon = 1.e-6f;
		if (R.X >= R.Y)
		{
			if (PAbs.Y < Epsilon)
			{
				return FVec2((P.X > 0.0f) ? R.X : -R.X, 0.0f);
			}
		}
		else
		{
			if (PAbs.X < Epsilon)
			{
				return FVec2(0.0f, (P.Y > 0.0f)? R.Y : -R.Y);
			}
		}

		// Iterate to find nearest point
		FVec2 R2 = R * R;
		FVec2 RP = R * PAbs;
		FReal T = FMath::Max(RP.X - R2.X, RP.Y - R2.Y);
		FVec2 D;
		for (int32 It = 0; It < MaxIts; ++It)
		{
			D = FVec2(1.0f / (T + R2.X), 1 / (T + R2.Y));
			FVec2 RPD = RP * D;

			FVec2 FV = RPD * RPD;
			FReal F = FV.X + FV.Y - 1.0f;

			if (F < Tolerance)
			{
				return (R2 * P) * D;
			}

			FReal DF = -2.0f * FVec2::DotProduct(FV, D);
			T = T - F / DF;
		}

		// Too many iterations - return current value clamped
		FVec2 S = (R2 * P) * D;
		FVec2 SN = S / R;	
		return S / SN.Size();
	}

	bool GetEllipticalAxisError(const FVec3& SwingAxisRot, const FVec3& EllipseNormal, const FVec3& TwistAxis, FVec3& AxisLocal, FReal& Error)
	{
		const FReal R2 = SwingAxisRot.SizeSquared();
		const FReal A = 1.0f - R2;
		const FReal B = 1.0f / (1.0f + R2);
		const FReal B2 = B * B;
		const FReal V1 = 2.0f * A * B2;
		const FVec3 V2 = FVec3(A, 2.0f * SwingAxisRot.Z, -2.0f * SwingAxisRot.Y);
		const FReal RD = FVec3::DotProduct(SwingAxisRot, EllipseNormal);
		const FReal DV1 = -4.0f * RD * (3.0f - R2) * B2 * B;
		const FVec3 DV2 = FVec3(-2.0f * RD, 2.0f * EllipseNormal.Z, -2.0f * EllipseNormal.Y);
		
		const FVec3 Line = V1 * V2 - FVec3(1, 0, 0);
		FVec3 Normal = V1 * DV2 + DV1 * V2;
		if (Utilities::NormalizeSafe(Normal))
		{
			AxisLocal = FVec3::CrossProduct(Line, Normal);
			Error = -FVec3::DotProduct(FVec3::CrossProduct(Line, AxisLocal), TwistAxis);
			return true;
		}
		return false;
	}

	void FPBDJointUtilities::GetSphericalAxisDelta(
		const FVec3& X0,
		const FVec3& X1,
		FVec3& Axis,
		FReal& Delta)
	{
		Axis = FVec3(0);
		Delta = 0;

		const FVec3 DX = X1 - X0;
		const FReal DXLen = DX.Size();
		if (DXLen > UE_KINDA_SMALL_NUMBER)
		{
			Axis = DX / DXLen;
			Delta = DXLen;
		}
	}


	void FPBDJointUtilities::GetCylindricalAxesDeltas(
		const FRotation3& R0,
		const FVec3& X0,
		const FVec3& X1,
		const int32 CylinderAxisIndex,
		FVec3& CylinderAxis,
		FReal& CylinderDelta,
		FVec3& RadialAxis,
		FReal& RadialDelta)
	{
		CylinderAxis = FVec3(0);
		CylinderDelta = 0;
		RadialAxis = FVec3(0);
		RadialDelta = 0;

		FVec3 DX = X1 - X0;

		const FMatrix33 R0M = R0.ToMatrix();
		CylinderAxis = R0M.GetAxis(CylinderAxisIndex);
		CylinderDelta = FVec3::DotProduct(DX, CylinderAxis);

		DX = DX - CylinderDelta * CylinderAxis;
		const FReal DXLen = DX.Size();
		if (DXLen > UE_KINDA_SMALL_NUMBER)
		{
			RadialAxis = DX / DXLen;
			RadialDelta = DXLen;
		}
	}


	void FPBDJointUtilities::GetPlanarAxisDelta(
		const FRotation3& R0,
		const FVec3& X0,
		const FVec3& X1,
		const int32 PlaneAxisIndex,
		FVec3& Axis,
		FReal& Delta)
	{
		const FMatrix33 R0M = R0.ToMatrix();
		Axis = R0M.GetAxis(PlaneAxisIndex);
		Delta = FVec3::DotProduct(X1 - X0, Axis);
	}

	void FPBDJointUtilities::DecomposeSwingTwistLocal(const FRotation3& R0, const FRotation3& R1, FRotation3& R01Swing, FRotation3& R01Twist)
	{
		const FRotation3 R01 = R0.Inverse() * R1;
		R01.ToSwingTwistX(R01Swing, R01Twist);
	}

	void FPBDJointUtilities::GetSwingTwistAngles(const FRotation3& R0, const FRotation3& R1, FReal& TwistAngle, FReal& Swing1Angle, FReal& Swing2Angle)
	{
		FRotation3 R01Twist, R01Swing;
		FPBDJointUtilities::DecomposeSwingTwistLocal(R0, R1, R01Swing, R01Twist);
		TwistAngle = GetTwistAngle(R01Twist);
		// @todo(ccaulfield): makes assumptions about Swing1 and Swing2 axes - fix this
		Swing1Angle = GetSwingAngleZ(R01Swing);
		Swing2Angle = GetSwingAngleY(R01Swing);
	}

	FReal FPBDJointUtilities::GetTwistAngle(const FRotation3& InTwist)
	{
		FRotation3 Twist = InTwist.GetNormalized();
		ensure(FMath::Abs(Twist.W) <= 1.0f);
		FReal Angle = Twist.GetAngle();
		if (Angle > UE_PI)
		{
			Angle = Angle - (FReal)2 * UE_PI;
		}
		if (Twist.X < 0.0f)
		{
			Angle = -Angle;
		}
		return Angle;
	}


	void FPBDJointUtilities::GetTwistAxisAngle(
		const FRotation3& R0,
		const FRotation3& R1,
		FVec3& Axis,
		FReal& Angle)
	{
		FRotation3 R01Twist, R01Swing;
		FPBDJointUtilities::DecomposeSwingTwistLocal(R0, R1, R01Swing, R01Twist);

		Axis = R1 * FJointConstants::TwistAxis();
		Angle = FPBDJointUtilities::GetTwistAngle(R01Twist);
	}


	void FPBDJointUtilities::GetConeAxisAngleLocal(
		const FRotation3& R0,
		const FRotation3& R1,
		const FReal AngleTolerance,
		FVec3& AxisLocal,
		FReal& Angle)
	{
		// Decompose rotation of body 1 relative to body 0 into swing and twist rotations, assuming twist is X axis
		FRotation3 R01Twist, R01Swing;
		FPBDJointUtilities::DecomposeSwingTwistLocal(R0, R1, R01Swing, R01Twist);

		R01Swing.ToAxisAndAngleSafe(AxisLocal, Angle, FJointConstants::Swing1Axis(), AngleTolerance);
		if (Angle > UE_PI)
		{
			Angle = Angle - (FReal)2 * UE_PI;
		}
	}

	void FPBDJointUtilities::GetCircularConeAxisErrorLocal(
		const FRotation3& R0,
		const FRotation3& R1,
		const FReal SwingLimit,
		FVec3& AxisLocal,
		FReal& Error)
	{
		FRotation3 R01Twist, R01Swing;
		FPBDJointUtilities::DecomposeSwingTwistLocal(R0, R1, R01Swing, R01Twist);

		FReal Angle = 0.0f;
		R01Swing.ToAxisAndAngleSafe(AxisLocal, Angle, FJointConstants::Swing1Axis(), 1.e-6f);

		Error = 0.0f;
		if (Angle > SwingLimit)
		{
			Error = Angle - SwingLimit;
		}
		else if (Angle < -SwingLimit)
		{
			Error = Angle + SwingLimit;
		}
	}

	void FPBDJointUtilities::GetEllipticalConeAxisErrorLocal(
		const FRotation3& R0,
		const FRotation3& R1,
		const FReal SwingLimitY,
		const FReal SwingLimitZ,
		FVec3& AxisLocal,
		FReal& Error)
	{
		if (FMath::IsNearlyEqual(SwingLimitY, SwingLimitZ, (FReal)1.e-3))
		{ 
			GetCircularConeAxisErrorLocal(R0, R1, SwingLimitY, AxisLocal, Error);
			return;
		}

		AxisLocal = FJointConstants::Swing1Axis();
		Error = 0.;

		FRotation3 R01Twist, R01Swing;
		FPBDJointUtilities::DecomposeSwingTwistLocal(R0, R1, R01Swing, R01Twist);

		const FVec2 SwingAngles = FVec2(GetSwingAngleY(R01Swing), GetSwingAngleZ(R01Swing));
		const FVec2 SwingLimits = FVec2(SwingLimitY, SwingLimitZ);

		// Transform onto a circle to see if we are within the ellipse
		const FVec2 CircleMappedAngles = SwingAngles / SwingLimits;
		if (CircleMappedAngles.SizeSquared() > 1.0f)
		{
			// Map the swing to a position on the elliptical limits
			const FVec2 ClampedSwingAngles = NearPointOnEllipse(SwingAngles, SwingLimits);

			// Get the ellipse normal
			const FVec2 ClampedNormal = ClampedSwingAngles / (SwingLimits * SwingLimits);

			// Calculate the axis and error
			const FVec3 TwistAxis = R01Swing.GetAxisX();
			const FVec3 SwingRotAxis = FVec3(0.0f, FMath::Tan(ClampedSwingAngles.X / 4.0f), FMath::Tan(ClampedSwingAngles.Y / 4.0f));
			const FVec3 EllipseNormal = FVec3(0.0f, ClampedNormal.X, ClampedNormal.Y);
			GetEllipticalAxisError(SwingRotAxis, EllipseNormal, TwistAxis, AxisLocal, Error);
		}
	}

	void FPBDJointUtilities::GetLockedSwingAxisAngle(
		const FRotation3& R0,
		const FRotation3& R1,
		const EJointAngularConstraintIndex SwingConstraintIndex,
		FVec3& Axis,
		FReal& Angle)
	{
		// NOTE: this differs from GetDualConeSwingAxisAngle in that it returns an axis with length Sin(SwingAngle)
		// and an Angle that is actually Sin(SwingAngle). This allows it to be used when we get closer to degenerate twist angles.
		FVec3 Twist1 = R1 * FJointConstants::TwistAxis();
		FVec3 Swing0 = R0 * FJointConstants::OtherSwingAxis(SwingConstraintIndex);
		Axis = FVec3::CrossProduct(Swing0, Twist1);
		Angle = -FVec3::DotProduct(Swing0, Twist1);
	}


	void FPBDJointUtilities::GetDualConeSwingAxisAngle(
		const FRotation3& R0,
		const FRotation3& R1,
		const EJointAngularConstraintIndex SwingConstraintIndex,
		FVec3& Axis,
		FReal& Angle)
	{
		FVec3 Twist1 = R1 * FJointConstants::TwistAxis();
		FVec3 Swing0 = R0 * FJointConstants::OtherSwingAxis(SwingConstraintIndex);
		Axis = FVec3::CrossProduct(Swing0, Twist1);
		Angle = 0.0f;
		if (Utilities::NormalizeSafe(Axis, UE_KINDA_SMALL_NUMBER))
		{
			FReal SwingTwistDot = FVec3::DotProduct(Swing0, Twist1);
			Angle = FMath::Asin(FMath::Clamp(-SwingTwistDot, FReal(-1), FReal(1)));
		}
	}


	void FPBDJointUtilities::GetSwingAxisAngle(
		const FRotation3& R0,
		const FRotation3& R1,
		const FReal AngleTolerance,
		const EJointAngularConstraintIndex SwingConstraintIndex,
		FVec3& Axis,
		FReal& Angle)
	{
		// Decompose rotation of body 1 relative to body 0 into swing and twist rotations, assuming twist is X axis
		FRotation3 R01Twist, R01Swing;
		FPBDJointUtilities::DecomposeSwingTwistLocal(R0, R1, R01Swing, R01Twist);
		const FReal R01SwingYorZ = (FJointConstants::AxisIndex(SwingConstraintIndex) == 2) ? R01Swing.Z : R01Swing.Y;	// Can't index a quat :(
		Angle = 4.0f * FMath::Atan2(R01SwingYorZ, (FReal)(1. + R01Swing.W));
		const FVec3& AxisLocal = (SwingConstraintIndex == EJointAngularConstraintIndex::Swing1) ? FJointConstants::Swing1Axis() : FJointConstants::Swing2Axis();
		Axis = R0 * AxisLocal;
	}


	void FPBDJointUtilities::GetLockedRotationAxes(const FRotation3& R0, const FRotation3& R1, FVec3& Axis0, FVec3& Axis1, FVec3& Axis2)
	{
		const FReal W0 = R0.W;
		const FReal W1 = R1.W;
		const FVec3 V0 = FVec3(R0.X, R0.Y, R0.Z);
		const FVec3 V1 = FVec3(R1.X, R1.Y, R1.Z);

		const FVec3 C = V1 * W0 + V0 * W1;
		const FReal D0 = W0 * W1;
		const FReal D1 = FVec3::DotProduct(V0, V1);
		const FReal D = D0 - D1;

		Axis0 = 0.5f * (V0 * V1.X + V1 * V0.X + FVec3(D, C.Z, -C.Y));
		Axis1 = 0.5f * (V0 * V1.Y + V1 * V0.Y + FVec3(-C.Z, D, C.X));
		Axis2 = 0.5f * (V0 * V1.Z + V1 * V0.Z + FVec3(C.Y, -C.X, D));

		// Handle degenerate case of 180 deg swing
		if (FMath::Abs(D0 + D1) < UE_SMALL_NUMBER)
		{
			const FReal Epsilon = UE_SMALL_NUMBER;
			Axis0.X += Epsilon;
			Axis1.Y += Epsilon;
			Axis2.Z += Epsilon;
		}
	}
	

	FReal FPBDJointUtilities::GetConeAngleLimit(
		const FPBDJointSettings& JointSettings,
		const FVec3& SwingAxisLocal,
		const FReal SwingAngle)
	{
		// Calculate swing limit for the current swing axis
		const FReal Swing1Limit = JointSettings.AngularLimits[(int32)EJointAngularConstraintIndex::Swing1];
		const FReal Swing2Limit = JointSettings.AngularLimits[(int32)EJointAngularConstraintIndex::Swing2];

		// Circular swing limit
		FReal SwingAngleMax = Swing1Limit;

		// Elliptical swing limit
		// @todo(ccaulfield): do elliptical constraints properly (axis is still for circular limit)
		if (!FMath::IsNearlyEqual(Swing1Limit, Swing2Limit, (FReal)UE_KINDA_SMALL_NUMBER))
		{
			// Map swing axis to ellipse and calculate limit for this swing axis
			const FReal DotSwing1 = FMath::Abs(FVec3::DotProduct(SwingAxisLocal, FJointConstants::Swing1Axis()));
			const FReal DotSwing2 = FMath::Abs(FVec3::DotProduct(SwingAxisLocal, FJointConstants::Swing2Axis()));
			SwingAngleMax = FMath::Sqrt(Swing1Limit * DotSwing1 * Swing1Limit * DotSwing1 + Swing2Limit * DotSwing2 * Swing2Limit * DotSwing2);
		}

		return SwingAngleMax;
	}

	bool FPBDJointUtilities::GetSoftLinearLimitEnabled(
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		return JointSettings.bSoftLinearLimitsEnabled && !bChaos_Joint_DisableSoftLimits;
	}

	bool FPBDJointUtilities::GetSoftTwistLimitEnabled(
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		return JointSettings.bSoftTwistLimitsEnabled && !bChaos_Joint_DisableSoftLimits;
	}

	bool FPBDJointUtilities::GetSoftSwingLimitEnabled(
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		return JointSettings.bSoftSwingLimitsEnabled && !bChaos_Joint_DisableSoftLimits;
	}

	FReal FPBDJointUtilities::GetLinearStiffness(
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		return (SolverSettings.LinearStiffnessOverride >= (FReal)0) ? SolverSettings.LinearStiffnessOverride : JointSettings.Stiffness;
	}

	FReal FPBDJointUtilities::GetSoftLinearStiffness(
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		return (SolverSettings.SoftLinearStiffnessOverride >= (FReal)0) ? SolverSettings.SoftLinearStiffnessOverride : JointSettings.SoftLinearStiffness;
	}

	FReal FPBDJointUtilities::GetSoftLinearDamping(
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		return (SolverSettings.SoftLinearDampingOverride >= (FReal)0) ? SolverSettings.SoftLinearDampingOverride : JointSettings.SoftLinearDamping;
	}

	FReal FPBDJointUtilities::GetTwistStiffness(
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		return (SolverSettings.TwistStiffnessOverride >= (FReal)0) ? SolverSettings.TwistStiffnessOverride : JointSettings.Stiffness;
	}

	FReal FPBDJointUtilities::GetSoftTwistStiffness(
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		return (SolverSettings.SoftTwistStiffnessOverride >= 0)? SolverSettings.SoftTwistStiffnessOverride : JointSettings.SoftTwistStiffness;
	}

	FReal FPBDJointUtilities::GetSoftTwistDamping(
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		return (SolverSettings.SoftTwistDampingOverride >= 0) ? SolverSettings.SoftTwistDampingOverride : JointSettings.SoftTwistDamping;
	}

	FReal FPBDJointUtilities::GetSwingStiffness(
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		return (SolverSettings.SwingStiffnessOverride >= (FReal)0) ? SolverSettings.SwingStiffnessOverride : JointSettings.Stiffness;
	}

	FReal FPBDJointUtilities::GetSoftSwingStiffness(
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		return (SolverSettings.SoftSwingStiffnessOverride >= 0) ? SolverSettings.SoftSwingStiffnessOverride : JointSettings.SoftSwingStiffness;
	}

	FReal FPBDJointUtilities::GetSoftSwingDamping(
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		return (SolverSettings.SoftSwingDampingOverride >= 0) ? SolverSettings.SoftSwingDampingOverride : JointSettings.SoftSwingDamping;
	}

	FReal FPBDJointUtilities::GetLinearDriveStiffness(
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings,
		const int32 AxisIndex)
	{
		if (JointSettings.bLinearPositionDriveEnabled[AxisIndex])
		{
			return (SolverSettings.LinearDriveStiffnessOverride >= 0.0f) ? SolverSettings.LinearDriveStiffnessOverride : JointSettings.LinearDriveStiffness[AxisIndex];
		}
		return 0.0f;
	}

	FReal FPBDJointUtilities::GetLinearDriveDamping(
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings,
		const int32 AxisIndex)
	{
		if (JointSettings.bLinearVelocityDriveEnabled[AxisIndex])
		{
			return (SolverSettings.LinearDriveDampingOverride >= 0.0f) ? SolverSettings.LinearDriveDampingOverride : JointSettings.LinearDriveDamping[AxisIndex];
		}
		return 0.0f;
	}

	FReal FPBDJointUtilities::GetAngularTwistDriveStiffness(
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		if (JointSettings.bAngularTwistPositionDriveEnabled)
		{
			return (SolverSettings.AngularDriveStiffnessOverride >= 0.0f) ? SolverSettings.AngularDriveStiffnessOverride : JointSettings.AngularDriveStiffness[(int)EJointAngularConstraintIndex::Twist];
		}
		return 0.0f;
	}

	FReal FPBDJointUtilities::GetAngularTwistDriveDamping(
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		if (JointSettings.bAngularTwistVelocityDriveEnabled)
		{
			return (SolverSettings.AngularDriveDampingOverride >= 0.0f) ? SolverSettings.AngularDriveDampingOverride : JointSettings.AngularDriveDamping[(int)EJointAngularConstraintIndex::Twist];
		}
		return 0.0f;
	}

	FReal FPBDJointUtilities::GetAngularTwistDriveTorqueLimit(
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		if (JointSettings.bAngularTwistVelocityDriveEnabled)
		{
			return JointSettings.AngularDriveMaxTorque[(int)EJointAngularConstraintIndex::Twist];
		}
		return 0.0f;
	}

	FReal FPBDJointUtilities::GetAngularSwingDriveStiffness(
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		if (JointSettings.bAngularSwingPositionDriveEnabled)
		{
			return (SolverSettings.AngularDriveStiffnessOverride >= 0.0f) ? SolverSettings.AngularDriveStiffnessOverride : JointSettings.AngularDriveStiffness[(int)EJointAngularConstraintIndex::Swing1];
		}
		return 0.0f;
	}

	FReal FPBDJointUtilities::GetAngularSwingDriveDamping(
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		if (JointSettings.bAngularSwingVelocityDriveEnabled)
		{
			return (SolverSettings.AngularDriveDampingOverride >= 0.0f) ? SolverSettings.AngularDriveDampingOverride : JointSettings.AngularDriveDamping[(int)EJointAngularConstraintIndex::Swing1];
		}
		return 0.0f;
	}

	FReal FPBDJointUtilities::GetAngularSwingDriveTorqueLimit(
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		if (JointSettings.bAngularSwingVelocityDriveEnabled)
		{
			return JointSettings.AngularDriveMaxTorque[(int)EJointAngularConstraintIndex::Swing1];
		}
		return 0.0f;
	}

	FReal FPBDJointUtilities::GetAngularSLerpDriveStiffness(
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		if (JointSettings.bAngularSLerpPositionDriveEnabled)
		{
			return (SolverSettings.AngularDriveStiffnessOverride >= 0.0f) ? SolverSettings.AngularDriveStiffnessOverride : JointSettings.AngularDriveStiffness[(int)EJointAngularConstraintIndex::Twist];
		}
		return 0.0f;
	}

	FReal FPBDJointUtilities::GetAngularSLerpDriveDamping(
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		if (JointSettings.bAngularSLerpVelocityDriveEnabled)
		{
			return (SolverSettings.AngularDriveDampingOverride >= 0.0f) ? SolverSettings.AngularDriveDampingOverride : JointSettings.AngularDriveDamping[(int)EJointAngularConstraintIndex::Twist];
		}
		return 0.0f;
	}

	FReal FPBDJointUtilities::GetLinearProjection(
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		return (SolverSettings.LinearProjectionOverride >= 0.0f) ? SolverSettings.LinearProjectionOverride : JointSettings.LinearProjection;
	}

	FReal FPBDJointUtilities::GetAngularProjection(
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		return (SolverSettings.AngularProjectionOverride >= 0.0f) ? SolverSettings.AngularProjectionOverride : JointSettings.AngularProjection;
	}

	bool FPBDJointUtilities::GetLinearSoftAccelerationMode(
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		return JointSettings.LinearSoftForceMode == EJointForceMode::Acceleration;
	}

	bool FPBDJointUtilities::GetAngularSoftAccelerationMode(
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		return JointSettings.AngularSoftForceMode == EJointForceMode::Acceleration;
	}

	bool FPBDJointUtilities::GetLinearDriveAccelerationMode(
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		return JointSettings.LinearDriveForceMode == EJointForceMode::Acceleration;
	}

	bool FPBDJointUtilities::GetAngularDriveAccelerationMode(
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		return JointSettings.AngularDriveForceMode == EJointForceMode::Acceleration;
	}

	FReal FPBDJointUtilities::GetShockPropagationInvMassScale(
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings)
	{
		// ShockProagation setting is a alpha. For an alpha of 0 we want an invmass scale of 1, and vice-versa
		return (SolverSettings.ShockPropagationOverride >= FReal(0)) ? (FReal(1) - SolverSettings.ShockPropagationOverride) : (FReal(1) - JointSettings.ShockPropagation);
	}


	FVec3 FPBDJointUtilities::ConditionInertia(const FVec3& InI, const FReal MaxRatio)
	{
		FReal IMin = InI.Min();
		FReal IMax = InI.Max();
		if ((MaxRatio > 0) && (IMin > 0))
		{
			FReal Ratio = IMax / IMin;
			if (Ratio > MaxRatio)
			{
				FReal MinIMin = IMax / MaxRatio;
				return FVec3(
					FMath::Lerp(MinIMin, IMax, (InI.X - IMin) / (IMax - IMin)),
					FMath::Lerp(MinIMin, IMax, (InI.Y - IMin) / (IMax - IMin)),
					FMath::Lerp(MinIMin, IMax, (InI.Z - IMin) / (IMax - IMin)));
			}
		}
		return InI;
	}

	
	FVec3 FPBDJointUtilities::ConditionParentInertia(const FVec3& IParent, const FVec3& IChild, const FReal MinRatio)
	{
		if (MinRatio > 0)
		{
			FReal IParentMax = IParent.Max();
			FReal IChildMax = IChild.Max();
			if ((IParentMax > 0) && (IChildMax > 0))
			{
				FReal Ratio = IParentMax / IChildMax;
				if (Ratio < MinRatio)
				{
					FReal Multiplier = MinRatio / Ratio;
					return IParent * Multiplier;
				}
			}
		}
		return IParent;
	}

	
	FReal FPBDJointUtilities::ConditionParentMass(const FReal MParent, const FReal MChild, const FReal MinRatio)
	{
		if ((MinRatio > 0) && (MParent > 0) && (MChild > 0))
		{
			FReal Ratio = MParent / MChild;
			if (Ratio < MinRatio)
			{
				FReal Multiplier = MinRatio / Ratio;
				return MParent * Multiplier;
			}
		}
		return MParent;
	}

	
	// @todo(ccaulfield): should also take into account the length of the joint connector to prevent over-rotation
	void FPBDJointUtilities::ConditionInverseMassAndInertia(
		const FReal& InInvMParent,
		const FReal& InInvMChild,
		const FVec3& InInvIParent,
		const FVec3& InInvIChild,
		const FReal MinParentMassRatio,
		const FReal MaxInertiaRatio,
		FReal& OutInvMParent,
		FReal& OutInvMChild,
		FVec3& OutInvIParent,
		FVec3& OutInvIChild)
	{
		FReal MParent = 0.0f;
		FVec3 IParent = FVec3(0);
		FReal MChild = 0.0f;
		FVec3 IChild = FVec3(0);

		// Set up inertia so that it is more uniform (reduce the maximum ratio of the inertia about each axis)
		if (InInvMParent > 0)
		{
			MParent = 1.0f / InInvMParent;
			IParent = ConditionInertia(FVec3(1.0f / InInvIParent.X, 1.0f / InInvIParent.Y, 1.0f / InInvIParent.Z), MaxInertiaRatio);
		}
		if (InInvMChild > 0)
		{
			MChild = 1.0f / InInvMChild;
			IChild = ConditionInertia(FVec3(1.0f / InInvIChild.X, 1.0f / InInvIChild.Y, 1.0f / InInvIChild.Z), MaxInertiaRatio);
		}

		// Set up relative mass and inertia so that the parent cannot be much lighter than the child
		if ((InInvMParent > 0) && (InInvMChild > 0))
		{
			MParent = ConditionParentMass(MParent, MChild, MinParentMassRatio);
			IParent = ConditionParentInertia(IParent, IChild, MinParentMassRatio);
		}

		// Map back to inverses
		if (InInvMParent > 0)
		{
			OutInvMParent = (FReal)1 / MParent;
			OutInvIParent = FVec3((FReal)1 / IParent.X, (FReal)1 / IParent.Y, (FReal)1 / IParent.Z);
		}
		else
		{
			OutInvMParent = 0.f;
			OutInvIParent = FVec3(0.f);
		}
		if (InInvMChild > 0)
		{
			OutInvMChild = (FReal)1 / MChild;
			OutInvIChild = FVec3((FReal)1 / IChild.X, (FReal)1 / IChild.Y, (FReal)1 / IChild.Z);
		}
		else
		{
			OutInvMChild = 0.f;
			OutInvIChild = FVec3(0.f);
		}
	}

	void FPBDJointUtilities::ConditionInverseMassAndInertia(
		FReal& InOutInvMParent,
		FReal& InOutInvMChild,
		FVec3& InOutInvIParent,
		FVec3& InOutInvIChild,
		const FReal MinParentMassRatio,
		const FReal MaxInertiaRatio)
	{
		ConditionInverseMassAndInertia(InOutInvMParent, InOutInvMChild, InOutInvIParent, InOutInvIChild, MinParentMassRatio, MaxInertiaRatio, InOutInvMParent, InOutInvMChild, InOutInvIParent, InOutInvIChild);
	}


	FVec3 FPBDJointUtilities::GetSphereLimitedPositionError(const FVec3& CX, const FReal Radius)
	{
		FReal CXLen = CX.Size();
		if (CXLen < Radius)
		{
			return FVec3(0, 0, 0);
		}
		else if (CXLen > UE_SMALL_NUMBER)
		{
			FVec3 Dir = CX / CXLen;
			return CX - Radius * Dir;
		}
		return CX;
	}


	FVec3 FPBDJointUtilities::GetCylinderLimitedPositionError(const FVec3& InCX, const FVec3& Axis, const FReal Limit, const EJointMotionType AxisMotion)
	{
		FVec3 CXAxis = FVec3::DotProduct(InCX, Axis) * Axis;
		FVec3 CXPlane = InCX - CXAxis;
		FReal CXPlaneLen = CXPlane.Size();
		if (AxisMotion == EJointMotionType::Free)
		{
			CXAxis = FVec3(0, 0, 0);
		}
		if (CXPlaneLen < Limit)
		{
			CXPlane = FVec3(0, 0, 0);
		}
		else if (CXPlaneLen > UE_KINDA_SMALL_NUMBER)
		{
			FVec3 Dir = CXPlane / CXPlaneLen;
			CXPlane = CXPlane - Limit * Dir;
		}
		return CXAxis + CXPlane;
	}


	FVec3 FPBDJointUtilities::GetLineLimitedPositionError(const FVec3& CX, const FVec3& Axis, const FReal Limit, const EJointMotionType AxisMotion)
	{
		FReal CXDist = FVec3::DotProduct(CX, Axis);
		if ((AxisMotion == EJointMotionType::Free) || (FMath::Abs(CXDist) < Limit))
		{
			return CX - CXDist * Axis;
		}
		else if (CXDist >= Limit)
		{
			return CX - Limit * Axis;
		}
		else
		{
			return CX + Limit * Axis;
		}
	}


	FVec3 FPBDJointUtilities::GetLimitedPositionError(const FPBDJointSettings& JointSettings, const FRotation3& R0, const FVec3& InCX)
	{
		// This function is only used for projection and is only relevant for hard limits.
		// Treat soft-limits as free for error calculation.
		const TVector<EJointMotionType, 3>& Motion =
		{
			((JointSettings.LinearMotionTypes[0] == EJointMotionType::Limited) && JointSettings.bSoftLinearLimitsEnabled) ? EJointMotionType::Free : JointSettings.LinearMotionTypes[0],
			((JointSettings.LinearMotionTypes[1] == EJointMotionType::Limited) && JointSettings.bSoftLinearLimitsEnabled) ? EJointMotionType::Free : JointSettings.LinearMotionTypes[1],
			((JointSettings.LinearMotionTypes[2] == EJointMotionType::Limited) && JointSettings.bSoftLinearLimitsEnabled) ? EJointMotionType::Free : JointSettings.LinearMotionTypes[2],
		};

		if ((Motion[0] == EJointMotionType::Locked) && (Motion[1] == EJointMotionType::Locked) && (Motion[2] == EJointMotionType::Locked))
		{
			return InCX;
		}
		else if ((Motion[0] == EJointMotionType::Limited) && (Motion[1] == EJointMotionType::Limited) && (Motion[2] == EJointMotionType::Limited))
		{
			// Spherical distance constraints
			return GetSphereLimitedPositionError(InCX, JointSettings.LinearLimit);
		}
		else if ((Motion[1] == EJointMotionType::Limited) && (Motion[2] == EJointMotionType::Limited))
		{
			// Circular Limit (X Axis)
			FVec3 Axis = R0 * FVec3(1, 0, 0);
			return GetCylinderLimitedPositionError(InCX, Axis, JointSettings.LinearLimit, Motion[0]);
		}
		else if ((Motion[0] == EJointMotionType::Limited) && (Motion[2] == EJointMotionType::Limited))
		{
			// Circular Limit (Y Axis)
			FVec3 Axis = R0 * FVec3(0, 1, 0);
			return GetCylinderLimitedPositionError(InCX, Axis, JointSettings.LinearLimit, Motion[1]);
		}
		else if ((Motion[0] == EJointMotionType::Limited) && (Motion[1] == EJointMotionType::Limited))
		{
			// Circular Limit (Z Axis)
			FVec3 Axis = R0 * FVec3(0, 0, 1);
			return GetCylinderLimitedPositionError(InCX, Axis, JointSettings.LinearLimit, Motion[2]);
		}
		else
		{
			// Line/Square/Cube Limits (no way to author square or cube limits, but would work if we wanted it)
			FVec3 CX = InCX;
			if (Motion[0] != EJointMotionType::Locked)
			{
				FVec3 Axis = R0 * FVec3(1, 0, 0);
				CX = GetLineLimitedPositionError(CX, Axis, JointSettings.LinearLimit, Motion[0]);
			}
			if (Motion[1] != EJointMotionType::Locked)
			{
				FVec3 Axis = R0 * FVec3(0, 1, 0);
				CX = GetLineLimitedPositionError(CX, Axis, JointSettings.LinearLimit, Motion[1]);
			}
			if (Motion[2] != EJointMotionType::Locked)
			{
				FVec3 Axis = R0 * FVec3(0, 0, 1);
				CX = GetLineLimitedPositionError(CX, Axis, JointSettings.LinearLimit, Motion[2]);
			}
			return CX;
		}
	}

	bool FPBDJointUtilities::GetLinearIsFree(const FPBDJointSettings& JointSettings)
	{
		return JointSettings.LinearMotionTypes == TVector<EJointMotionType, 3>(EJointMotionType::Free)
			&& JointSettings.bLinearPositionDriveEnabled == TVector<bool, 3>(false) && bExcludeFreeJointInertiaConditioning;
	}
}
