// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/PBDJointConstraints.h"
#include "Chaos/ChaosDebugDraw.h"
#include "Chaos/DebugDrawQueue.h"
#include "Chaos/Island/IslandManager.h"
#include "Chaos/Joint/ChaosJointLog.h"
#include "Chaos/Joint/ColoringGraph.h"
#include "Chaos/Joint/JointConstraintsCVars.h"
#include "Chaos/Joint/PBDJointContainerSolver.h"
#include "Chaos/Particle/ParticleUtilities.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/PBDJointConstraintUtilities.h"
#include "Chaos/Utilities.h"
#include "ChaosLog.h"
#include "ChaosStats.h"

#include "HAL/IConsoleManager.h"

namespace Chaos
{
	DECLARE_CYCLE_STAT(TEXT("Joints::Sort"), STAT_Joints_Sort, STATGROUP_ChaosJoint);
	DECLARE_CYCLE_STAT(TEXT("Joints::PrepareTick"), STAT_Joints_PrepareTick, STATGROUP_ChaosJoint);
	DECLARE_CYCLE_STAT(TEXT("Joints::UnprepareTick"), STAT_Joints_UnprepareTick, STATGROUP_ChaosJoint);
	DECLARE_CYCLE_STAT(TEXT("Joints::Gather"), STAT_Joints_Gather, STATGROUP_ChaosJoint);
	DECLARE_CYCLE_STAT(TEXT("Joints::Scatter"), STAT_Joints_Scatter, STATGROUP_ChaosJoint);
	DECLARE_CYCLE_STAT(TEXT("Joints::Apply"), STAT_Joints_Apply, STATGROUP_ChaosJoint);
	DECLARE_CYCLE_STAT(TEXT("Joints::ApplyPushOut"), STAT_Joints_ApplyPushOut, STATGROUP_ChaosJoint);
	DECLARE_CYCLE_STAT(TEXT("Joints::ApplyProjection"), STAT_Joints_ApplyProjection, STATGROUP_ChaosJoint);


	//
	// Constraint Handle
	//

	
	FPBDJointConstraintHandle::FPBDJointConstraintHandle()
		: bLinearPlasticityInitialized(false)
		, bAngularPlasticityInitialized(false)
	{
	}

	
	FPBDJointConstraintHandle::FPBDJointConstraintHandle(FConstraintContainer* InConstraintContainer, int32 InConstraintIndex)
		: TIndexedContainerConstraintHandle<FPBDJointConstraints>(InConstraintContainer, InConstraintIndex)
		, bLinearPlasticityInitialized(false)
		, bAngularPlasticityInitialized(false)

	{
	}

	
	void FPBDJointConstraintHandle::CalculateConstraintSpace(FVec3& OutXa, FMatrix33& OutRa, FVec3& OutXb, FMatrix33& OutRb) const
	{
		ConcreteContainer()->CalculateConstraintSpace(ConstraintIndex, OutXa, OutRa, OutXb, OutRb);
	}


	int32 FPBDJointConstraintHandle::GetConstraintIsland() const
	{
		return ConcreteContainer()->GetConstraintIsland(ConstraintIndex);
	}


	int32 FPBDJointConstraintHandle::GetConstraintLevel() const
	{
		return ConcreteContainer()->GetConstraintLevel(ConstraintIndex);
	}


	int32 FPBDJointConstraintHandle::GetConstraintColor() const
	{
		return ConcreteContainer()->GetConstraintColor(ConstraintIndex);
	}

	bool FPBDJointConstraintHandle::IsConstraintBroken() const
	{
		return ConcreteContainer()->IsConstraintBroken(ConstraintIndex);
	}

	bool FPBDJointConstraintHandle::IsConstraintBreaking() const
	{
		return ConcreteContainer()->IsConstraintBreaking(ConstraintIndex);
	}

	void FPBDJointConstraintHandle::ClearConstraintBreaking()
	{
		ConcreteContainer()->ClearConstraintBreaking(ConstraintIndex);
	}

	bool FPBDJointConstraintHandle::IsConstraintViolating() const
	{
		return ConcreteContainer()->IsConstraintViolating(ConstraintIndex);
	}

	void FPBDJointConstraintHandle::ClearConstraintViolating()
	{
		ConcreteContainer()->ClearConstraintViolating(ConstraintIndex);
	}

	bool FPBDJointConstraintHandle::IsDriveTargetChanged() const
	{
		return ConcreteContainer()->IsDriveTargetChanged(ConstraintIndex);
	}

	void FPBDJointConstraintHandle::ClearDriveTargetChanged()
	{
		ConcreteContainer()->ClearDriveTargetChanged(ConstraintIndex);
	}

	bool FPBDJointConstraintHandle::IsConstraintEnabled() const
	{
		return ConcreteContainer()->IsConstraintEnabled(ConstraintIndex);
	}

	FVec3 FPBDJointConstraintHandle::GetLinearImpulse() const
	{
		return ConcreteContainer()->GetConstraintLinearImpulse(ConstraintIndex);
	}

	FVec3 FPBDJointConstraintHandle::GetAngularImpulse() const
	{
		return ConcreteContainer()->GetConstraintAngularImpulse(ConstraintIndex);
	}

	float FPBDJointConstraintHandle::GetLinearViolation() const
	{
		return ConcreteContainer()->GetConstraintLinearViolation(ConstraintIndex);
	}

	float FPBDJointConstraintHandle::GetAngularViolation() const
	{
		return ConcreteContainer()->GetConstraintAngularViolation(ConstraintIndex);
	}

	ESyncState FPBDJointConstraintHandle::SyncState() const
	{
		return ConcreteContainer()->GetConstraintSyncState(ConstraintIndex);
	}

	void FPBDJointConstraintHandle::SetSyncState(ESyncState SyncState)
	{
		ConcreteContainer()->SetConstraintSyncState(ConstraintIndex, SyncState);
	}

	void FPBDJointConstraintHandle::SetEnabledDuringResim(bool bEnabled)
	{
		ConcreteContainer()->SetConstraintEnabledDuringResim(ConstraintIndex, bEnabled);
	}

	bool FPBDJointConstraintHandle::IsEnabledDuringResim() const
	{
		return ConcreteContainer()->IsConstraintEnabledDuringResim(ConstraintIndex);
	}

	EResimType FPBDJointConstraintHandle::ResimType() const
	{
		return ConcreteContainer()->GetConstraintResimType(ConstraintIndex);
	}

	void FPBDJointConstraintHandle::SetResimType(EResimType ResimType)
	{
		ConcreteContainer()->SetConstraintResimType(ConstraintIndex, ResimType);
	}

	constexpr int32 ConstraintChildIndex = 0;
	constexpr int32 ConstraintParentIndex = 1;

	FPBDJointSettings& FPBDJointConstraintHandle::GetConstraintSettingsInternal() 
	{ 
		return ConcreteContainer()->ConstraintSettings[ConstraintIndex]; 
	}

	void FPBDJointConstraintHandle::SetParentConnectorLocation(const FVec3 Location)
	{
		GetConstraintSettingsInternal().ConnectorTransforms[ConstraintParentIndex].SetLocation(Location);
	}

	void FPBDJointConstraintHandle::SetParentConnectorRotation(const FQuat Rotation)
	{
		GetConstraintSettingsInternal().ConnectorTransforms[ConstraintParentIndex].SetRotation(Rotation);
	}

	void FPBDJointConstraintHandle::SetChildConnectorLocation(const FVec3 Location)
	{
		GetConstraintSettingsInternal().ConnectorTransforms[ConstraintChildIndex].SetLocation(Location);
	}

	void FPBDJointConstraintHandle::SetChildConnectorRotation(const FQuat Rotation)
	{
		GetConstraintSettingsInternal().ConnectorTransforms[ConstraintChildIndex].SetRotation(Rotation);
	}

	void FPBDJointConstraintHandle::SetLinearDrivePositionTarget(const FVec3 Target)
	{
		GetConstraintSettingsInternal().LinearDrivePositionTarget = Target;
	}

	void FPBDJointConstraintHandle::SetAngularDrivePositionTarget(const FQuat Target)
	{
		GetConstraintSettingsInternal().AngularDrivePositionTarget = Target;
	}

	void FPBDJointConstraintHandle::SetLinearDriveVelocityTarget(const FVec3 Target)
	{
		GetConstraintSettingsInternal().LinearDriveVelocityTarget = Target;
	}

	void FPBDJointConstraintHandle::SetAngularDriveVelocityTarget(const FVec3 Target)
	{
		GetConstraintSettingsInternal().AngularDriveVelocityTarget = Target;
	}

	void FPBDJointConstraintHandle::SetLinearDriveStiffness(const FVec3 Stiffness)
	{
		GetConstraintSettingsInternal().LinearDriveStiffness = Stiffness;
	}

	void FPBDJointConstraintHandle::SetLinearDriveDamping(const FVec3 Damping)
	{
		GetConstraintSettingsInternal().LinearDriveDamping = Damping;
	}

	void FPBDJointConstraintHandle::SetLinearDriveMaxForce(const FVec3 MaxForce)
	{
		GetConstraintSettingsInternal().LinearDriveMaxForce = MaxForce;
	}

	void FPBDJointConstraintHandle::SetLinearDriveForceMode(EJointForceMode ForceMode)
	{
		GetConstraintSettingsInternal().LinearDriveForceMode = ForceMode;
	}

	void FPBDJointConstraintHandle::SetAngularDriveStiffness(const FVec3 Stiffness)
	{
		GetConstraintSettingsInternal().AngularDriveStiffness = Stiffness;
	}

	void FPBDJointConstraintHandle::SetAngularDriveDamping(const FVec3 Damping)
	{
		GetConstraintSettingsInternal().AngularDriveDamping = Damping;
	}

	void FPBDJointConstraintHandle::SetAngularDriveMaxTorque(const FVec3 MaxTorque)
	{
		GetConstraintSettingsInternal().AngularDriveMaxTorque = MaxTorque;
	}

	void FPBDJointConstraintHandle::SetAngularDriveForceMode(EJointForceMode ForceMode)
	{
		GetConstraintSettingsInternal().AngularDriveForceMode = ForceMode;
	}

	void FPBDJointConstraintHandle::SetCollisionEnabled(const bool bCollisionEnabled)
	{
		GetConstraintSettingsInternal().bCollisionEnabled = bCollisionEnabled;
	}

	void FPBDJointConstraintHandle::SetParentInvMassScale(const FReal ParentInvMassScale)
	{
		GetConstraintSettingsInternal().ParentInvMassScale = ParentInvMassScale;
	}

	void FPBDJointConstraintHandle::SetDriveParams(
		const FVec3 LinearStiffness, const FVec3 LinearDamping, const FVec3 MaxForce,
		const FVec3 AngularStiffness, const FVec3 AngularDamping, const FVec3 MaxTorque)
	{
		FPBDJointSettings& Settings = GetConstraintSettingsInternal();
		Settings.LinearDriveStiffness = LinearStiffness;
		Settings.LinearDriveDamping = LinearDamping;
		Settings.LinearDriveMaxForce = MaxForce;
		Settings.AngularDriveStiffness = AngularStiffness;
		Settings.AngularDriveDamping = AngularDamping;
		Settings.AngularDriveMaxTorque = MaxTorque;
		Settings.bLinearPositionDriveEnabled[0] = LinearStiffness.X > 0;
		Settings.bLinearPositionDriveEnabled[1] = LinearStiffness.Y > 0;
		Settings.bLinearPositionDriveEnabled[2] = LinearStiffness.Z > 0;
		Settings.bLinearVelocityDriveEnabled[0] = LinearDamping.X > 0;
		Settings.bLinearVelocityDriveEnabled[1] = LinearDamping.Y > 0;
		Settings.bLinearVelocityDriveEnabled[2] = LinearDamping.Z > 0;
		Settings.bAngularSwingPositionDriveEnabled = AngularStiffness.X > 0;
		Settings.bAngularTwistPositionDriveEnabled = AngularStiffness.Y > 0;
		Settings.bAngularSLerpPositionDriveEnabled = AngularStiffness.Z > 0;
		Settings.bAngularSwingVelocityDriveEnabled = AngularDamping.X > 0;
		Settings.bAngularTwistVelocityDriveEnabled = AngularDamping.Y > 0;
		Settings.bAngularSLerpVelocityDriveEnabled = AngularDamping.Z > 0;
	}

	const FPBDJointSettings& FPBDJointConstraintHandle::GetSettings() const
	{
		return ConcreteContainer()->GetConstraintSettings(ConstraintIndex);
	}


	void FPBDJointConstraintHandle::SetJointState(const FPBDJointState& NewState)
	{
		ConcreteContainer()->SetConstraintState(ConstraintIndex, NewState);
	}

	void FPBDJointConstraintHandle::SetSettings(const FPBDJointSettings& InSettings)
	{
		// Track whether plasticity was already initialized before this call.
		// We only want to restore the buffered targets for subsequent updates,
		// not during the initial constraint setup.
		const bool bWasLinearPlasticityInitialized = bLinearPlasticityInitialized;
		const bool bWasAngularPlasticityInitialized = bAngularPlasticityInitialized;

		// buffer the previous targets so plasticity can reuse them
		FVec3 LinearTarget = GetSettings().LinearDrivePositionTarget;
		FRotation3 AngularTarget = GetSettings().AngularDrivePositionTarget;
		if (!bLinearPlasticityInitialized && !FMath::IsNearlyEqual(InSettings.LinearPlasticityLimit, FLT_MAX))
		{
			bLinearPlasticityInitialized = true;
		}
		if (!bAngularPlasticityInitialized && !FMath::IsNearlyEqual(InSettings.AngularPlasticityLimit, FLT_MAX))
		{
			bAngularPlasticityInitialized = true;
		}

		ConcreteContainer()->SetConstraintSettings(ConstraintIndex, InSettings);


		// transfer the previous targets when controlled by plasticity
		if (bWasLinearPlasticityInitialized)
		{
			ConcreteContainer()->SetLinearDrivePositionTarget(ConstraintIndex,LinearTarget);
		}
		if (bWasAngularPlasticityInitialized)
		{
			ConcreteContainer()->SetAngularDrivePositionTarget(ConstraintIndex,AngularTarget);
		}
	}

	FParticlePair FPBDJointConstraintHandle::GetConstrainedParticles() const
	{ 
		return ConcreteContainer()->GetConstrainedParticles(ConstraintIndex);
	}

	void FPBDJointConstraintHandle::SetConstraintEnabled(bool bInEnabled)
	{
		ConcreteContainer()->SetConstraintEnabled(ConstraintIndex, bInEnabled);
	}


	//
	// Constraint Settings
	//

	
	FPBDJointSettings::FPBDJointSettings()
		: Stiffness(1)
		, LinearProjection(0)
		, AngularProjection(0)
		, ShockPropagation(0)
		, TeleportDistance(0)
		, TeleportAngle(0)
		, ParentInvMassScale(1)
		, bCollisionEnabled(true)
		, bProjectionEnabled(false)
		, bShockPropagationEnabled(false)
		, bMassConditioningEnabled(true)
		, bUseLinearSolver(true)
		, LinearMotionTypes({ EJointMotionType::Locked, EJointMotionType::Locked, EJointMotionType::Locked })
		, LinearLimit(UE_MAX_FLT)
		, AngularMotionTypes({ EJointMotionType::Free, EJointMotionType::Free, EJointMotionType::Free })
		, AngularLimits(FVec3(UE_MAX_FLT, UE_MAX_FLT, UE_MAX_FLT))
		, bSoftLinearLimitsEnabled(false)
		, bSoftTwistLimitsEnabled(false)
		, bSoftSwingLimitsEnabled(false)
		, LinearSoftForceMode(EJointForceMode::Acceleration)
		, AngularSoftForceMode(EJointForceMode::Acceleration)
		, SoftLinearStiffness(0)
		, SoftLinearDamping(0)
		, SoftTwistStiffness(0)
		, SoftTwistDamping(0)
		, SoftSwingStiffness(0)
		, SoftSwingDamping(0)
		, LinearRestitution(0)
		, TwistRestitution(0)
		, SwingRestitution(0)
		, LinearContactDistance(0)
		, TwistContactDistance(0)
		, SwingContactDistance(0)
		, LinearDrivePositionTarget(FVec3(0, 0, 0))
		, LinearDriveVelocityTarget(FVec3(0, 0, 0))
		, bLinearPositionDriveEnabled(TVector<bool, 3>(false, false, false))
		, bLinearVelocityDriveEnabled(TVector<bool, 3>(false, false, false))
		, LinearDriveForceMode(EJointForceMode::Acceleration)
		, LinearDriveStiffness(FVec3(0))
		, LinearDriveDamping(FVec3(0))
		, LinearDriveMaxForce(FVec3(UE_MAX_FLT))
		, AngularDrivePositionTarget(FRotation3::FromIdentity())
		, AngularDriveVelocityTarget(FVec3(0, 0, 0))
		, bAngularSLerpPositionDriveEnabled(false)
		, bAngularSLerpVelocityDriveEnabled(false)
		, bAngularTwistPositionDriveEnabled(false)
		, bAngularTwistVelocityDriveEnabled(false)
		, bAngularSwingPositionDriveEnabled(false)
		, bAngularSwingVelocityDriveEnabled(false)
		, AngularDriveForceMode(EJointForceMode::Acceleration)
		, AngularDriveStiffness(FVec3(0))
		, AngularDriveDamping(FVec3(0))
		, AngularDriveMaxTorque(FVec3(UE_MAX_FLT))
		, LinearBreakForce(UE_MAX_FLT)
		, LinearViolationCallbackThreshold(UE_MAX_FLT)
		, LinearPlasticityLimit(UE_MAX_FLT)
		, LinearPlasticityType(EPlasticityType::Free)
		, LinearPlasticityInitialDistanceSquared(UE_MAX_FLT)
		, AngularBreakTorque(UE_MAX_FLT)
		, AngularViolationCallbackThreshold(UE_MAX_FLT)
		, AngularPlasticityLimit(UE_MAX_FLT)
		, ContactTransferScale(0.f)
		, UserData(nullptr)
	{
	}


	void FPBDJointSettings::Sanitize()
	{
		// Disable soft joints for locked dofs
		if ((LinearMotionTypes[0] == EJointMotionType::Locked) && (LinearMotionTypes[1] == EJointMotionType::Locked) && (LinearMotionTypes[2] == EJointMotionType::Locked))
		{
			bSoftLinearLimitsEnabled = false;
		}
		if (AngularMotionTypes[(int32)EJointAngularConstraintIndex::Twist] == EJointMotionType::Locked)
		{
			bSoftTwistLimitsEnabled = false;
		}
		if ((AngularMotionTypes[(int32)EJointAngularConstraintIndex::Swing1] == EJointMotionType::Locked) && (AngularMotionTypes[(int32)EJointAngularConstraintIndex::Swing2] == EJointMotionType::Locked))
		{
			bSoftSwingLimitsEnabled = false;
		}

		// Reset limits if they won't be used (means we don't have to check if limited/locked in a few cases).
		// A side effect: if we enable a constraint, we need to reset the value of the limit.
		if ((LinearMotionTypes[0] != EJointMotionType::Limited) && (LinearMotionTypes[1] != EJointMotionType::Limited) && (LinearMotionTypes[2] != EJointMotionType::Limited))
		{
			LinearLimit = 0;
		}
		if (AngularMotionTypes[(int32)EJointAngularConstraintIndex::Twist] != EJointMotionType::Limited)
		{
			AngularLimits[(int32)EJointAngularConstraintIndex::Twist] = 0;
		}
		if (AngularMotionTypes[(int32)EJointAngularConstraintIndex::Swing1] != EJointMotionType::Limited)
		{
			AngularLimits[(int32)EJointAngularConstraintIndex::Swing1] = 0;
		}
		if (AngularMotionTypes[(int32)EJointAngularConstraintIndex::Swing2] != EJointMotionType::Limited)
		{
			AngularLimits[(int32)EJointAngularConstraintIndex::Swing2] = 0;
		}

		// If we have a zero degree limit angle, lock the joint, or set a non-zero limit (to avoid division by zero in axis calculations)
		const FReal MinAngularLimit = 0.01f;
		if ((AngularMotionTypes[(int32)EJointAngularConstraintIndex::Twist] == EJointMotionType::Limited) && (AngularLimits[(int32)EJointAngularConstraintIndex::Twist] < MinAngularLimit))
		{
			if (bSoftTwistLimitsEnabled)
			{
				AngularLimits[(int32)EJointAngularConstraintIndex::Twist] = MinAngularLimit;
			}
			else
			{
				AngularMotionTypes[(int32)EJointAngularConstraintIndex::Twist] = EJointMotionType::Locked;
			}
		}
		if ((AngularMotionTypes[(int32)EJointAngularConstraintIndex::Swing1] == EJointMotionType::Limited) && (AngularLimits[(int32)EJointAngularConstraintIndex::Swing1] < MinAngularLimit))
		{
			if (bSoftSwingLimitsEnabled)
			{
				AngularLimits[(int32)EJointAngularConstraintIndex::Swing1] = MinAngularLimit;
			}
			else
			{
				AngularMotionTypes[(int32)EJointAngularConstraintIndex::Swing1] = EJointMotionType::Locked;
			}
		}
		if ((AngularMotionTypes[(int32)EJointAngularConstraintIndex::Swing2] == EJointMotionType::Limited) && (AngularLimits[(int32)EJointAngularConstraintIndex::Swing2] < MinAngularLimit))
		{
			if (bSoftSwingLimitsEnabled)
			{
				AngularLimits[(int32)EJointAngularConstraintIndex::Swing2] = MinAngularLimit;
			}
			else
			{
				AngularMotionTypes[(int32)EJointAngularConstraintIndex::Swing2] = EJointMotionType::Locked;
			}
		}

		// SLerp drive is only allowed if no angular dofs are locked
		if (bAngularSLerpPositionDriveEnabled || bAngularSLerpVelocityDriveEnabled)
		{
			if ((AngularMotionTypes[(int32)EJointAngularConstraintIndex::Twist] == EJointMotionType::Locked)
				|| (AngularMotionTypes[(int32)EJointAngularConstraintIndex::Swing1] == EJointMotionType::Locked)
				|| (AngularMotionTypes[(int32)EJointAngularConstraintIndex::Swing2] == EJointMotionType::Locked))
			{
				bAngularSLerpPositionDriveEnabled = false;
				bAngularSLerpVelocityDriveEnabled = false;
			}
		}
	}

	
	FPBDJointState::FPBDJointState()
		: Island(INDEX_NONE)
		, Level(INDEX_NONE)
		, Color(INDEX_NONE)
		, IslandSize(0)
		, bDisabled(false)
		, bBroken(false)
		, bBreaking(false)
		, bDriveTargetChanged(false), LinearImpulse(FVec3(0))
		, AngularImpulse(FVec3(0))
	{
	}


	//
	// Solver Settings
	//

	
	FPBDJointSolverSettings::FPBDJointSolverSettings()
		: SwingTwistAngleTolerance(1.0e-6f)
		, PositionTolerance(0)
		, AngleTolerance(0)
		, MinParentMassRatio(0)
		, MaxInertiaRatio(0)
		, MinSolverStiffness(1)
		, MaxSolverStiffness(1)
		, NumIterationsAtMaxSolverStiffness(1)
		, NumShockPropagationIterations(0)
		, bUseSimd(false)
		, bSortEnabled(false)
		, bSolvePositionLast(true)
		, bUsePositionBasedDrives(true)
		, bEnableTwistLimits(true)
		, bEnableSwingLimits(true)
		, bEnableDrives(true)
		, LinearStiffnessOverride(-1)
		, TwistStiffnessOverride(-1)
		, SwingStiffnessOverride(-1)
		, LinearProjectionOverride(-1)
		, AngularProjectionOverride(-1)
		, ShockPropagationOverride(-1)
		, LinearDriveStiffnessOverride(-1)
		, LinearDriveDampingOverride(-1)
		, AngularDriveStiffnessOverride(-1)
		, AngularDriveDampingOverride(-1)
		, SoftLinearStiffnessOverride(-1)
		, SoftLinearDampingOverride(-1)
		, SoftTwistStiffnessOverride(-1)
		, SoftTwistDampingOverride(-1)
		, SoftSwingStiffnessOverride(-1)
		, SoftSwingDampingOverride(-1)
	{
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	
	FPBDJointSolverSettings::FPBDJointSolverSettings(const FPBDJointSolverSettings& InSettings) = default;

	FPBDJointSolverSettings& FPBDJointSolverSettings::operator=(const FPBDJointSolverSettings& Other) = default;
	
	PRAGMA_ENABLE_DEPRECATION_WARNINGS


	//
	// Constraint Container
	//

	
	FPBDJointConstraints::FPBDJointConstraints()
		: Base(FConstraintContainerHandle::StaticType())
		, Settings()
		, bJointsDirty(false)
	{
	}

	
	FPBDJointConstraints::~FPBDJointConstraints()
	{
	}

	TUniquePtr<FConstraintContainerSolver> FPBDJointConstraints::CreateSceneSolver(const int32 Priority)
	{
		if (bUseLinearSolver)
		{
			return MakeUnique<Private::TPBDJointContainerSolver<Chaos::FPBDJointCachedSolver>>(*this, Priority);
		}
		else
		{
			return MakeUnique<Private::TPBDJointContainerSolver<Chaos::FPBDJointSolver>>(*this, Priority);
		}
	}

	TUniquePtr<FConstraintContainerSolver> FPBDJointConstraints::CreateGroupSolver(const int32 Priority)
	{
		if (bUseLinearSolver)
		{
			return MakeUnique<Private::TPBDJointContainerSolver<Chaos::FPBDJointCachedSolver>>(*this, Priority);
		}
		else
		{
			return MakeUnique<Private::TPBDJointContainerSolver<Chaos::FPBDJointSolver>>(*this, Priority);
		}
	}
	
	const FPBDJointSolverSettings& FPBDJointConstraints::GetSettings() const
	{
		return Settings;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	void FPBDJointConstraints::SetSettings(const FPBDJointSolverSettings& InSettings)
	{
		Settings = InSettings;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	int32 FPBDJointConstraints::NumConstraints() const
	{
		return ConstraintParticles.Num();
	}

	void FPBDJointConstraints::GetConstrainedParticleIndices(const int32 ConstraintIndex, int32& Index0, int32& Index1) const
	{
		// In solvers we need Particle0 to be the parent particle but ConstraintInstance has Particle1 as the parent, so by default
		// we need to flip the indices before we pass them to the solver. 

		Index0 = 1;
		Index1 = 0;
	}

	typename FPBDJointConstraints::FConstraintContainerHandle* FPBDJointConstraints::AddConstraint(const FParticlePair& InConstrainedParticles, const FRigidTransform3& WorldConstraintFrame)
	{
		FPBDJointSettings JointSettings;
		JointSettings.ConnectorTransforms[0] = FRigidTransform3(
			WorldConstraintFrame.GetTranslation() - InConstrainedParticles[0]->GetX(),
			WorldConstraintFrame.GetRotation() * InConstrainedParticles[0]->GetR().Inverse()
			);
		JointSettings.ConnectorTransforms[1] = FRigidTransform3(
			WorldConstraintFrame.GetTranslation() - InConstrainedParticles[1]->GetX(),
			WorldConstraintFrame.GetRotation() * InConstrainedParticles[1]->GetR().Inverse()
			);
		return AddConstraint(InConstrainedParticles, JointSettings);
	}

	
	typename FPBDJointConstraints::FConstraintContainerHandle* FPBDJointConstraints::AddConstraint(const FParticlePair& InConstrainedParticles, const FTransformPair& InConnectorTransforms)
	{
		FPBDJointSettings JointSettings;
		JointSettings.ConnectorTransforms = InConnectorTransforms;
		return AddConstraint(InConstrainedParticles, JointSettings);
	}

	
	typename FPBDJointConstraints::FConstraintContainerHandle* FPBDJointConstraints::AddConstraint(const FParticlePair& InConstrainedParticles, const FPBDJointSettings& InConstraintSettings)
	{
		bJointsDirty = true;

		int ConstraintIndex = Handles.Num();
		Handles.Add(HandleAllocator.AllocHandle(this, ConstraintIndex));
		ConstraintParticles.Add(InConstrainedParticles);
		ConstraintStates.Add(FPBDJointState());

		ConstraintSettings.AddDefaulted();
		SetConstraintSettings(ConstraintIndex, InConstraintSettings);

		// If our particle(s) are disabled, so is the constraint for now. It will get enabled if both particles get enabled.
		const bool bStartDisabled = FConstGenericParticleHandle(InConstrainedParticles[0])->Disabled() || FConstGenericParticleHandle(InConstrainedParticles[1])->Disabled();
		ConstraintStates[ConstraintIndex].bDisabled = bStartDisabled;

		return Handles.Last();
	}

	void FPBDJointConstraints::SetConstraintState(int32 ConstraintIndex, const FPBDJointState& InState)
	{
		ConstraintStates[ConstraintIndex] = InState;
	}

	void FPBDJointConstraints::RemoveConstraint(int ConstraintIndex)
	{
		bJointsDirty = true;

		FConstraintContainerHandle* ConstraintHandle = Handles[ConstraintIndex];
		if (ConstraintHandle != nullptr)
		{
			if (ConstraintParticles[ConstraintIndex][0] != nullptr)
			{
				ConstraintParticles[ConstraintIndex][0]->RemoveConstraintHandle(ConstraintHandle);
			}
			if (ConstraintParticles[ConstraintIndex][1] != nullptr)
			{
				ConstraintParticles[ConstraintIndex][1]->RemoveConstraintHandle(ConstraintHandle);
			}

			// Release the handle for the freed constraint
			HandleAllocator.FreeHandle(ConstraintHandle);
			Handles[ConstraintIndex] = nullptr;
		}

		// Swap the last constraint into the gap to keep the array packed
		ConstraintParticles.RemoveAtSwap(ConstraintIndex);
		ConstraintSettings.RemoveAtSwap(ConstraintIndex);
		ConstraintStates.RemoveAtSwap(ConstraintIndex);
		Handles.RemoveAtSwap(ConstraintIndex);

		// Update the handle for the constraint that was moved
		if (ConstraintIndex < Handles.Num())
		{
			Handles[ConstraintIndex]->SetConstraintIndex(ConstraintIndex);
		}
	}

	
	void FPBDJointConstraints::DisconnectConstraints(const TSet<TGeometryParticleHandle<FReal, 3>*>& RemovedParticles)
	{
		for (TGeometryParticleHandle<FReal, 3>*RemovedParticle : RemovedParticles)
		{
			for (FConstraintHandle* ConstraintHandle : RemovedParticle->ParticleConstraints())
			{
				if (FPBDJointConstraintHandle* JointHandle = ConstraintHandle->As<FPBDJointConstraintHandle>())
				{
					if (!JointHandle->IsInContainer(this))
					{
						continue;
					}

					JointHandle->SetEnabled(false); // constraint lifespan is managed by the proxy

					int ConstraintIndex = JointHandle->GetConstraintIndex();
					if (ConstraintIndex != INDEX_NONE)
					{
						if (ConstraintParticles[ConstraintIndex][0] == RemovedParticle)
						{
							ConstraintParticles[ConstraintIndex][0] = nullptr;
						}
						if (ConstraintParticles[ConstraintIndex][1] == RemovedParticle)
						{
							ConstraintParticles[ConstraintIndex][1] = nullptr;
						}
					}
				}
			}
		}
	}


	void FPBDJointConstraints::SortConstraints()
	{
		// Sort constraints so that constraints with lower level (closer to a kinematic joint) are first
		// @todo(chaos): this is only needed for RBAN and should be moved to an RBAN specific solver
		// (so we should split TPBDJointContainerSolver into Scene and Group versions)
		SCOPE_CYCLE_COUNTER(STAT_Joints_Sort);

		TArray<int32> SortedIndices;
		SortedIndices.Reserve(Handles.Num());
		for (int32 JointIndex = 0; JointIndex < GetNumConstraints(); ++JointIndex)
		{
			SortedIndices.Add(JointIndex);
		}
		SortedIndices.StableSort([this](const int32& LIndex, const int32& RIndex)
			{
				if (GetConstraintIsland(LIndex) != GetConstraintIsland(RIndex))
				{
					return GetConstraintIsland(LIndex) < GetConstraintIsland(RIndex);
				}
				else if (GetConstraintLevel(LIndex) != GetConstraintLevel(RIndex))
				{
					return GetConstraintLevel(LIndex) < GetConstraintLevel(RIndex);
				}
				return GetConstraintColor(LIndex) < GetConstraintColor(RIndex);
			});

		TArray<FPBDJointConstraintHandle*> SortedHandles;
		TArray<FPBDJointSettings> SortedConstraintSettings;
		TArray<FParticlePair> SortedConstraintParticles;
		TArray<FPBDJointState> SortedConstraintStates;
		SortedHandles.Reserve(SortedIndices.Num());
		SortedConstraintSettings.Reserve(SortedIndices.Num());
		SortedConstraintParticles.Reserve(SortedIndices.Num());
		SortedConstraintStates.Reserve(SortedIndices.Num());

		for (int32 SortedConstraintIndex = 0; SortedConstraintIndex < SortedIndices.Num(); ++SortedConstraintIndex)
		{
			int32 UnsortedConstraintIndex = SortedIndices[SortedConstraintIndex];

			SortedHandles.Add(Handles[UnsortedConstraintIndex]);
			SortedConstraintSettings.Add(ConstraintSettings[UnsortedConstraintIndex]);
			SortedConstraintParticles.Add(ConstraintParticles[UnsortedConstraintIndex]);
			SortedConstraintStates.Add(ConstraintStates[UnsortedConstraintIndex]);

			Handles[UnsortedConstraintIndex]->SetConstraintIndex(SortedConstraintIndex);
		}

		Swap(Handles, SortedHandles);
		Swap(ConstraintSettings, SortedConstraintSettings);
		Swap(ConstraintParticles, SortedConstraintParticles);
		Swap(ConstraintStates, SortedConstraintStates);
	}


	bool FPBDJointConstraints::IsConstraintEnabled(int32 ConstraintIndex) const
	{
		return !ConstraintStates[ConstraintIndex].bDisabled;
	}

	bool FPBDJointConstraints::IsConstraintBroken(int32 ConstraintIndex) const
	{
		return ConstraintStates[ConstraintIndex].bBroken;
	}

	bool FPBDJointConstraints::IsConstraintBreaking(int32 ConstraintIndex) const
	{
		return ConstraintStates[ConstraintIndex].bBreaking;
	}

	void FPBDJointConstraints::ClearConstraintBreaking(int32 ConstraintIndex)
	{
		ConstraintStates[ConstraintIndex].bBreaking = false;
	}

	bool FPBDJointConstraints::IsConstraintViolating(int32 ConstraintIndex) const
	{
		return ConstraintStates[ConstraintIndex].bViolating;
	}

	void FPBDJointConstraints::ClearConstraintViolating(int32 ConstraintIndex)
	{
		ConstraintStates[ConstraintIndex].bViolating = false;
	}

	bool FPBDJointConstraints::IsDriveTargetChanged(int32 ConstraintIndex) const
	{
		return ConstraintStates[ConstraintIndex].bDriveTargetChanged;
	}

	void FPBDJointConstraints::ClearDriveTargetChanged(int32 ConstraintIndex)
	{
		ConstraintStates[ConstraintIndex].bDriveTargetChanged = false;
	}

	void FPBDJointConstraints::SetConstraintEnabled(int32 ConstraintIndex, bool bEnabled)
	{
		if (bEnabled)
		{ 
			if (ConstraintStates[ConstraintIndex].bDisabled)
			{
				const FGenericParticleHandle Particle0 = FGenericParticleHandle(ConstraintParticles[ConstraintIndex][0]);
				const FGenericParticleHandle Particle1 = FGenericParticleHandle(ConstraintParticles[ConstraintIndex][1]);

				// only enable constraint if the particles are valid and not disabled
				// and if the constraint is not broken
				if (Particle0->Handle() != nullptr && !Particle0->Disabled()
					&& Particle1->Handle() != nullptr && !Particle1->Disabled()
					&& !IsConstraintBroken(ConstraintIndex))
				{
					ConstraintStates[ConstraintIndex].bDisabled = false;
				}
			}
		}
		else
		{ 
			// desirable to allow disabling no matter what state the endpoints
			ConstraintStates[ConstraintIndex].bDisabled = true;
		}
	}

	void FPBDJointConstraints::SetConstraintBroken(int32 ConstraintIndex, bool bBroken)
	{
		ConstraintStates[ConstraintIndex].bBroken = bBroken;
	}

	void FPBDJointConstraints::SetConstraintBreaking(int32 ConstraintIndex, bool bBreaking)
	{
		ConstraintStates[ConstraintIndex].bBreaking = bBreaking;
	}

	void FPBDJointConstraints::SetDriveTargetChanged(int32 ConstraintIndex, bool bTargetChanged)
	{
		ConstraintStates[ConstraintIndex].bDriveTargetChanged = bTargetChanged;
	}

	void FPBDJointConstraints::BreakConstraint(int32 ConstraintIndex)
	{
		SetConstraintEnabled(ConstraintIndex, false);
		SetConstraintBroken(ConstraintIndex, true);
		SetConstraintBreaking(ConstraintIndex, true);
		if (BreakCallback)
		{
			BreakCallback(Handles[ConstraintIndex]);
		}
	}

	void FPBDJointConstraints::FixConstraint(int32 ConstraintIndex)
	{
		SetConstraintBroken(ConstraintIndex, false);
		SetConstraintEnabled(ConstraintIndex, true);
	}

	
	void FPBDJointConstraints::SetBreakCallback(const FJointBreakCallback& Callback)
	{
		BreakCallback = Callback;
	}


	void FPBDJointConstraints::ClearBreakCallback()
	{
		BreakCallback = nullptr;
	}


	const typename FPBDJointConstraints::FConstraintContainerHandle* FPBDJointConstraints::GetConstraintHandle(int32 ConstraintIndex) const
	{
		return Handles[ConstraintIndex];
	}

	
	typename FPBDJointConstraints::FConstraintContainerHandle* FPBDJointConstraints::GetConstraintHandle(int32 ConstraintIndex)
	{
		return Handles[ConstraintIndex];
	}

	
	const FParticlePair& FPBDJointConstraints::GetConstrainedParticles(int32 ConstraintIndex) const
	{
		return ConstraintParticles[ConstraintIndex];
	}

	
	const FPBDJointSettings& FPBDJointConstraints::GetConstraintSettings(int32 ConstraintIndex) const
	{
		return ConstraintSettings[ConstraintIndex];
	}


	void FPBDJointConstraints::SetConstraintSettings(int32 ConstraintIndex, const FPBDJointSettings& InConstraintSettings)
	{
		ConstraintSettings[ConstraintIndex] = InConstraintSettings;
		ConstraintSettings[ConstraintIndex].Sanitize();
	}

	void FPBDJointConstraints::SetLinearDrivePositionTarget(int32 ConstraintIndex, FVec3 InLinearDrivePositionTarget)
	{
		ConstraintSettings[ConstraintIndex].LinearDrivePositionTarget = InLinearDrivePositionTarget;
	}

	void FPBDJointConstraints::SetAngularDrivePositionTarget(int32 ConstraintIndex, FRotation3 InAngularDrivePositionTarget)
	{
		ConstraintSettings[ConstraintIndex].AngularDrivePositionTarget = InAngularDrivePositionTarget;
	}

	void FPBDJointConstraints::SetUseLinearSolver(int32 ConstraintIndex, bool InbUseLinearSolver)
	{
		ConstraintSettings[ConstraintIndex].bUseLinearSolver = InbUseLinearSolver;
	}

	int32 FPBDJointConstraints::GetConstraintIsland(int32 ConstraintIndex) const
	{
		return ConstraintStates[ConstraintIndex].Island;
	}


	int32 FPBDJointConstraints::GetConstraintLevel(int32 ConstraintIndex) const
	{
		return ConstraintStates[ConstraintIndex].Level;
	}


	int32 FPBDJointConstraints::GetConstraintColor(int32 ConstraintIndex) const
	{
		return ConstraintStates[ConstraintIndex].Color;
	}

	FVec3 FPBDJointConstraints::GetConstraintLinearImpulse(int32 ConstraintIndex) const
	{
		return ConstraintStates[ConstraintIndex].LinearImpulse;
	}

	FVec3 FPBDJointConstraints::GetConstraintAngularImpulse(int32 ConstraintIndex) const
	{
		return ConstraintStates[ConstraintIndex].AngularImpulse;
	}

	float FPBDJointConstraints::GetConstraintLinearViolation(int32 ConstraintIndex) const
	{
		return ConstraintStates[ConstraintIndex].LinearViolation;
	}

	float FPBDJointConstraints::GetConstraintAngularViolation(int32 ConstraintIndex) const
	{
		return ConstraintStates[ConstraintIndex].AngularViolation;
	}

	ESyncState FPBDJointConstraints::GetConstraintSyncState(int32 ConstraintIndex) const
	{
		return ConstraintStates[ConstraintIndex].SyncState;
	}

	void FPBDJointConstraints::SetConstraintSyncState(int32 ConstraintIndex, ESyncState SyncState)
	{
		ConstraintStates[ConstraintIndex].SyncState = SyncState;
	}

	void FPBDJointConstraints::SetConstraintEnabledDuringResim(int32 ConstraintIndex, bool bEnabled)
	{
		ConstraintStates[ConstraintIndex].bEnabledDuringResim = bEnabled;
	}

	bool FPBDJointConstraints::IsConstraintEnabledDuringResim(int32 ConstraintIndex) const
	{
		return ConstraintStates[ConstraintIndex].bEnabledDuringResim;
	}

	EResimType FPBDJointConstraints::GetConstraintResimType(int32 ConstraintIndex) const
	{
		return ConstraintStates[ConstraintIndex].ResimType;
	}

	void FPBDJointConstraints::SetConstraintResimType(int32 ConstraintIndex, EResimType ResimType)
	{
		ConstraintStates[ConstraintIndex].ResimType = ResimType;
	}

	void FPBDJointConstraints::PrepareTick()
	{
		SCOPE_CYCLE_COUNTER(STAT_Joints_PrepareTick);

		if (bJointsDirty)
		{
			if (Settings.bSortEnabled)
			{
				ColorConstraints();
				SortConstraints();
			}

			bJointsDirty = false;
		}
	}

	void FPBDJointConstraints::UnprepareTick()
	{
		SCOPE_CYCLE_COUNTER(STAT_Joints_UnprepareTick);
	}
	
	void FPBDJointConstraints::CalculateConstraintSpace(int32 ConstraintIndex, FVec3& OutX0, FMatrix33& OutR0, FVec3& OutX1, FMatrix33& OutR1) const
	{
		int32 Index0, Index1;
		GetConstrainedParticleIndices(ConstraintIndex, Index0, Index1);
		FGenericParticleHandle Particle0 = FGenericParticleHandle(ConstraintParticles[ConstraintIndex][Index0]);
		FGenericParticleHandle Particle1 = FGenericParticleHandle(ConstraintParticles[ConstraintIndex][Index1]);
		const FVec3 P0 = FParticleUtilities::GetCoMWorldPosition(Particle0);
		const FRotation3 Q0 = FParticleUtilities::GetCoMWorldRotation(Particle0);
		const FVec3 P1 = FParticleUtilities::GetCoMWorldPosition(Particle1);
		const FRotation3 Q1 = FParticleUtilities::GetCoMWorldRotation(Particle1);
		const FRigidTransform3& XL0 = FParticleUtilities::ParticleLocalToCoMLocal(Particle0, ConstraintSettings[ConstraintIndex].ConnectorTransforms[Index0]);
		const FRigidTransform3& XL1 = FParticleUtilities::ParticleLocalToCoMLocal(Particle1, ConstraintSettings[ConstraintIndex].ConnectorTransforms[Index1]);

		OutX0 = P0 + Q0 * XL0.GetTranslation();
		OutX1 = P1 + Q1 * XL1.GetTranslation();
		OutR0 = FRotation3(Q0 * XL0.GetRotation()).ToMatrix();
		OutR1 = FRotation3(Q1 * XL1.GetRotation()).ToMatrix();
	}

	void FPBDJointConstraints::AddConstraintsToGraph(Private::FPBDIslandManager& IslandManager)
	{
		for (int32 ConstraintIndex = 0; ConstraintIndex < GetNumConstraints(); ++ConstraintIndex)
		{
			FPBDJointConstraintHandle* ConstraintHandle = GetConstraintHandle(ConstraintIndex);
			check(ConstraintHandle != nullptr);

			const bool bIsInGraph = ConstraintHandle->IsInConstraintGraph();
			const bool bShouldBeInGraph = ShouldBeInGraph(ConstraintIndex);

			if (bShouldBeInGraph && !bIsInGraph)
			{
				IslandManager.AddConstraint(ContainerId, ConstraintHandle, GetConstrainedParticles(ConstraintIndex));
			}
			else if (bIsInGraph && !bShouldBeInGraph)
			{
				IslandManager.RemoveConstraint(ConstraintHandle);
			}
		}
	}

	void FPBDJointConstraints::SetSolverResults(const int32 ConstraintIndex, const FVec3& LinearImpulse, const FVec3& AngularImpulse, const float LinearViolation, const float AngularViolation, const bool bIsBroken, const bool bIsViolating, const FSolverBody* SolverBody0, const FSolverBody* SolverBody1)
	{
		ConstraintStates[ConstraintIndex].LinearImpulse = LinearImpulse;
		ConstraintStates[ConstraintIndex].AngularImpulse = AngularImpulse;

		if (bIsBroken)
		{
			BreakConstraint(ConstraintIndex);
		}

		ConstraintStates[ConstraintIndex].bViolating = bIsViolating;
		ConstraintStates[ConstraintIndex].LinearViolation = LinearViolation;
		ConstraintStates[ConstraintIndex].AngularViolation = AngularViolation;

		if ((SolverBody0 != nullptr) && (SolverBody1 != nullptr))
		{
			ApplyPlasticityLimits(ConstraintIndex, *SolverBody0, *SolverBody1);
		}
	}

	bool FPBDJointConstraints::ShouldBeInGraph(const int32 ConstraintIndex) const
	{
		if (!IsConstraintEnabled(ConstraintIndex))
		{
			return false;
		}

		const FGenericParticleHandle Particle0 = FGenericParticleHandle(ConstraintParticles[ConstraintIndex][0]);
		const FGenericParticleHandle Particle1 = FGenericParticleHandle(ConstraintParticles[ConstraintIndex][1]);

		// Must have 2 particles
		if (!Particle0.IsValid() || !Particle1.IsValid())
		{
			return false;
		}

		// Both particles must be active
		if (Particle0->Disabled() || Particle1->Disabled())
		{
			return false;
		}

		// Must have at least one dynamic particle
		if (!Particle0->IsDynamic() && !Particle1->IsDynamic())
		{
			return false;
		}

		// NOTE: Joints between sleeping particles (or a kinematic and sleeper) must stay in the graph 
		// so that when a kinematic particle moves it will wake the attached particle/island

		return true;
	}

	void FPBDJointConstraints::ApplyPlasticityLimits(int32 ConstraintIndex, const FSolverBody& SolverBody0, const FSolverBody& SolverBody1)
	{
		FPBDJointSettings& JointSettings = ConstraintSettings[ConstraintIndex];
		const bool bHasLinearPlasticityLimit = JointSettings.LinearPlasticityLimit != FLT_MAX;
		const bool bHasAngularPlasticityLimit = JointSettings.AngularPlasticityLimit != FLT_MAX;
		const bool bHasPlasticityLimits = bHasLinearPlasticityLimit || bHasAngularPlasticityLimit;
		if (!bHasPlasticityLimits)
		{
			return;
		}

		if (!Settings.bEnableDrives)
		{
			return;
		}

		// @todo(chaos): this should be done when the joint transforms are initialized
		// Plasticity should not be turned on in the middle of simulation.
		if (bHasLinearPlasticityLimit)
		{
			const bool bIsCOMDistanceInitialized = !FMath::IsNearlyEqual(JointSettings.LinearPlasticityInitialDistanceSquared, (FReal)FLT_MAX);
			if (!bIsCOMDistanceInitialized)
			{
				// Joint plasticity is based on the distance of one of the moment arms of the joint. Typically, plasticity
				// will get setup from the joint pivot to the child COM (centor of mass), so that is found first. However, when 
				// the pivot is at the child COM then we fall back to the distance between thge pivot and parent COM.
				ConstraintSettings[ConstraintIndex].LinearPlasticityInitialDistanceSquared = JointSettings.ConnectorTransforms[0].GetTranslation().SizeSquared();
				if (FMath::IsNearlyZero(ConstraintSettings[ConstraintIndex].LinearPlasticityInitialDistanceSquared))
				{
					ConstraintSettings[ConstraintIndex].LinearPlasticityInitialDistanceSquared = JointSettings.ConnectorTransforms[1].GetTranslation().SizeSquared();
				}
				// @todo(chaos): move this to validation
				ensureMsgf(!FMath::IsNearlyZero(ConstraintSettings[ConstraintIndex].LinearPlasticityInitialDistanceSquared), TEXT("Plasticity made inactive due to Zero length difference between parent and child rigid body."));
			}
		}

		int32 Index0, Index1;
		GetConstrainedParticleIndices(ConstraintIndex, Index0, Index1);
		{
			FGenericParticleHandle Particle0 = FGenericParticleHandle(ConstraintParticles[ConstraintIndex][Index0]);
			FGenericParticleHandle Particle1 = FGenericParticleHandle(ConstraintParticles[ConstraintIndex][Index1]);
			if (Particle0->Disabled() || Particle1->Disabled())
			{
				return;
			}
		}

		const FTransformPair& ConstraintFramesLocal = JointSettings.ConnectorTransforms;
		FTransformPair ConstraintFramesGlobal(ConstraintFramesLocal[Index0] * FRigidTransform3(SolverBody0.ActorP(), SolverBody0.ActorQ()), ConstraintFramesLocal[Index1] * FRigidTransform3(SolverBody1.ActorP(), SolverBody1.ActorQ()));
		FQuat Q1 = ConstraintFramesGlobal[1].GetRotation();
		Q1.EnforceShortestArcWith(ConstraintFramesGlobal[0].GetRotation());
		ConstraintFramesGlobal[1].SetRotation(Q1);

		if (bHasLinearPlasticityLimit)
		{
			FVec3 LinearDisplacement = ConstraintFramesGlobal[0].InverseTransformPositionNoScale(ConstraintFramesGlobal[1].GetTranslation());

			// @todo(chaos): still need to warn against the case where all position drives are not enabled or all dimensions are locked. Warning should print out the joint names and should only print out once to avoid spamming.
			for (int32 Axis = 0; Axis < 3; Axis++)
			{
				if (!JointSettings.bLinearPositionDriveEnabled[Axis] || JointSettings.LinearMotionTypes[Axis] == EJointMotionType::Locked)
				{
					LinearDisplacement[Axis] = 0;
				}
			}
			// Assuming that the dimensions which are locked or have no targets are 0. in LinearDrivePositionTarget
			FReal LinearPlasticityDistanceThreshold = JointSettings.LinearPlasticityLimit * JointSettings.LinearPlasticityLimit * JointSettings.LinearPlasticityInitialDistanceSquared;
			if ((LinearDisplacement - JointSettings.LinearDrivePositionTarget).SizeSquared() > LinearPlasticityDistanceThreshold)
			{
				if (JointSettings.LinearPlasticityType == EPlasticityType::Free)
				{
					JointSettings.LinearDrivePositionTarget = LinearDisplacement;
					SetDriveTargetChanged(ConstraintIndex, true);
				}
				else // EPlasticityType::Shrink || EPlasticityType::Grow
				{
					// Shrink and Grow are based on the distance between the joint pivot and the child. 
					// Note, if the pivot is located at the COM of the child then shrink will not do anything. 
					FVec3 StartDelta = ConstraintFramesLocal[Index1].InverseTransformPositionNoScale(JointSettings.LinearDrivePositionTarget);
					FVec3 CurrentDelta = ConstraintFramesGlobal[Index1].InverseTransformPositionNoScale(SolverBody1.P());

					if (JointSettings.LinearPlasticityType == EPlasticityType::Shrink && CurrentDelta.SizeSquared() < StartDelta.SizeSquared())
					{
						JointSettings.LinearDrivePositionTarget = LinearDisplacement;
						SetDriveTargetChanged(ConstraintIndex, true);
					}
					else if (JointSettings.LinearPlasticityType == EPlasticityType::Grow && CurrentDelta.SizeSquared() > StartDelta.SizeSquared())
					{
						JointSettings.LinearDrivePositionTarget = LinearDisplacement;
						SetDriveTargetChanged(ConstraintIndex, true);
					}
				}
			}
		}
		if (bHasAngularPlasticityLimit)
		{
			FRotation3 Swing, Twist; FPBDJointUtilities::DecomposeSwingTwistLocal(ConstraintFramesGlobal[0].GetRotation(), ConstraintFramesGlobal[1].GetRotation(), Swing, Twist);

			// @todo(chaos): still need to warn against the case where all position drives are not enabled or all dimensions are locked. Warning should print out the joint names and should only print out once to avoid spamming.
			if ((!JointSettings.bAngularSLerpPositionDriveEnabled && !JointSettings.bAngularTwistPositionDriveEnabled) || JointSettings.AngularMotionTypes[(int32)EJointAngularConstraintIndex::Twist] == EJointMotionType::Locked)
			{
				Twist = FRotation3::Identity;
			}
			// @todo(chaos): clamp rotation if only swing1(swing2) is locked
			if ((!JointSettings.bAngularSLerpPositionDriveEnabled && !JointSettings.bAngularSwingPositionDriveEnabled) || (JointSettings.AngularMotionTypes[(int32)EJointAngularConstraintIndex::Swing1] == EJointMotionType::Locked && JointSettings.AngularMotionTypes[(int32)EJointAngularConstraintIndex::Swing2] == EJointMotionType::Locked))
			{
				Swing = FRotation3::Identity;
			}

			const FRotation3 AngularDisplacement = Swing * Twist;
			// Assuming that the dimensions which are locked or have no targets are 0. in AngularDrivePositionTarget
			const FReal AngleRad = JointSettings.AngularDrivePositionTarget.AngularDistance(AngularDisplacement);
			if (AngleRad > JointSettings.AngularPlasticityLimit)
			{
				JointSettings.AngularDrivePositionTarget = AngularDisplacement;
				SetDriveTargetChanged(ConstraintIndex, true);
			}
		}
	}

	// Assign an Island, Level and Color to each constraint. Constraints must be processed in Level order, but
	// constraints of the same color are independent and can be processed in parallel (SIMD or Task)
	// NOTE: Constraints are the Vertices in this graph, and Edges connect constraints sharing a Particle. 
	// This makes the coloring of constraints simpler, but might not be what you expect so keep that in mind! 
	void FPBDJointConstraints::ColorConstraints()
	{
		// Add a Vertex for all constraints involving at least one dynamic body
		// Maintain a map from Constraint Index to Vertex Index
		FColoringGraph Graph;
		TArray<int32> ConstraintVertices; // Map of ConstraintIndex -> VertexIndex
		Graph.ReserveVertices(NumConstraints());
		ConstraintVertices.SetNumZeroed(NumConstraints());
		for (int32 ConstraintIndex = 0; ConstraintIndex < NumConstraints(); ++ConstraintIndex)
		{
			if (ConstraintStates[ConstraintIndex].bDisabled)
			{
				continue;
			}

			TPBDRigidParticleHandle<FReal, 3>* Particle0 = ConstraintParticles[ConstraintIndex][0]->CastToRigidParticle();
			TPBDRigidParticleHandle<FReal, 3>* Particle1 = ConstraintParticles[ConstraintIndex][1]->CastToRigidParticle();
			bool IsParticle0Dynamic = (Particle0 != nullptr) && (Particle0->ObjectState() == EObjectStateType::Dynamic || Particle0->ObjectState() == EObjectStateType::Sleeping);
			bool IsParticle1Dynamic = (Particle1 != nullptr) && (Particle1->ObjectState() == EObjectStateType::Dynamic || Particle1->ObjectState() == EObjectStateType::Sleeping);

			bool bContainsDynamic = IsParticle0Dynamic || IsParticle1Dynamic;
			if (bContainsDynamic)
			{
				ConstraintVertices[ConstraintIndex] = Graph.AddVertex();

				// Set kinematic-connected constraints to level 0 to initialize level calculation
				bool bContainsKinematic = !IsParticle0Dynamic || !IsParticle1Dynamic;
				if (bContainsKinematic)
				{
					Graph.SetVertexLevel(ConstraintVertices[ConstraintIndex], 0);
				}
			}
			else
			{
				// Constraint has no dynamics
				// This shouldn't happen often, but particles can change from dynamic to kinematic
				// and back again witout destroying joints, so it needs to be supported
				ConstraintVertices[ConstraintIndex] = INDEX_NONE;
			}
		}

		// Build a map of particles to constraints. We ignore non-dynamic particles since
		// two constraints that share only a static/kinematic particle will not interact.
		TMap<const FGeometryParticleHandle*, TArray<int32>> ParticleConstraints; // Map of ParticleHandle -> Constraint Indices involving the particle
		for (int32 ConstraintIndex = 0; ConstraintIndex < NumConstraints(); ++ConstraintIndex)
		{
			if (ConstraintStates[ConstraintIndex].bDisabled)
			{
				continue;
			}

			const FConstGenericParticleHandle Particle0 = ConstraintParticles[ConstraintIndex][0];
			const FConstGenericParticleHandle Particle1 = ConstraintParticles[ConstraintIndex][1];
			
			if (Particle0->IsDynamic())
			{
				ParticleConstraints.FindOrAdd(Particle0->Handle()).Add(ConstraintIndex);
			}
			if (Particle1->IsDynamic())
			{
				ParticleConstraints.FindOrAdd(Particle1->Handle()).Add(ConstraintIndex);
			}
		}

		// Connect constraints that share a dynamic particle
		// Algorithm:
		//		Loop over particles
		//			Loop over all constraint pairs on that particle
		//				Add an edge to connect the constraints
		//
		Graph.ReserveEdges((ParticleConstraints.Num() * (ParticleConstraints.Num() - 1)) / 2);
		for (auto& ParticleConstraintsElement : ParticleConstraints)
		{
			// Loop over constraint pairs connected to the particle
			// Visit each pair only once (see inner loop indexing)
			const TArray<int32>& ParticleConstraintIndices = ParticleConstraintsElement.Value;
			const int32 NumParticleConstraintIndices = ParticleConstraintIndices.Num();
			for (int32 ParticleConstraintIndex0 = 0; ParticleConstraintIndex0 < NumParticleConstraintIndices; ++ParticleConstraintIndex0)
			{
				const int32 ConstraintIndex0 = ParticleConstraintIndices[ParticleConstraintIndex0];
				const int32 VertexIndex0 = ConstraintVertices[ConstraintIndex0];
				if(VertexIndex0 == INDEX_NONE)
				{
					// Constraint has no dynamics
					continue;
				}

				for (int32 ParticleConstraintIndex1 = ParticleConstraintIndex0 + 1; ParticleConstraintIndex1 < NumParticleConstraintIndices; ++ParticleConstraintIndex1)
				{
					const int32 ConstraintIndex1 = ParticleConstraintIndices[ParticleConstraintIndex1];
					const int32 VertexIndex1 = ConstraintVertices[ConstraintIndex1];
					if(VertexIndex1 == INDEX_NONE)
					{
						// Constraint has no dynamics
						continue;
					}
					Graph.AddEdge(VertexIndex0, VertexIndex1);
				}
			}
		}

		// Colorize the graph
		Graph.Islandize();
		Graph.Levelize();
		Graph.Colorize();

		// Set the constraint colors
		for (int32 ConstraintIndex = 0; ConstraintIndex < NumConstraints(); ++ConstraintIndex)
		{
			if ( ConstraintStates[ConstraintIndex].bDisabled) continue;

			int32 VertexIndex = ConstraintVertices[ConstraintIndex];
			ConstraintStates[ConstraintIndex].Island = Graph.GetVertexIsland(VertexIndex);
			ConstraintStates[ConstraintIndex].IslandSize = Graph.GetVertexIslandSize(VertexIndex);
			ConstraintStates[ConstraintIndex].Level = Graph.GetVertexLevel(VertexIndex);
			ConstraintStates[ConstraintIndex].Color = Graph.GetVertexColor(VertexIndex);
		}
	}

	FPBDJointCombinedConstraints::FPBDJointCombinedConstraints()
	{
		LinearConstraints.SetUseLinearSolver(true);
		NonLinearConstraints.SetUseLinearSolver(false);
	}

	//This function moves one constraint info from linear constraint into nonlinear constraints
	void FPBDJointCombinedConstraints::ConvertLinearConstraintToNonLinear(const int32 LinearConstraintIndex)
	{
		if (!ensureMsgf(LinearConstraintIndex > INDEX_NONE && LinearConstraintIndex < LinearConstraints.NumConstraints(), TEXT("Linear constraint accessed is out of bounds!")))
		{
			return;
		}
		NonLinearConstraints.bJointsDirty = true;

		int NonLinearConstraintsNum = NonLinearConstraints.Handles.Num();
		NonLinearConstraints.Handles.Add(MoveTemp(LinearConstraints.Handles[LinearConstraintIndex]));
		NonLinearConstraints.ConstraintParticles.Add(MoveTemp(LinearConstraints.ConstraintParticles[LinearConstraintIndex]));
		NonLinearConstraints.ConstraintStates.Add(MoveTemp(LinearConstraints.ConstraintStates[LinearConstraintIndex]));
		NonLinearConstraints.ConstraintSettings.Add(MoveTemp(LinearConstraints.ConstraintSettings[LinearConstraintIndex]));
		NonLinearConstraints.ConstraintSettings[NonLinearConstraintsNum].bUseLinearSolver = false;
		NonLinearConstraints.Handles[NonLinearConstraintsNum]->SetConstraintIndex(NonLinearConstraintsNum);
		NonLinearConstraints.Handles[NonLinearConstraintsNum]->SetConstraintContainer(&NonLinearConstraints);

		LinearConstraints.Handles.RemoveAtSwap(LinearConstraintIndex);
		LinearConstraints.ConstraintParticles.RemoveAtSwap(LinearConstraintIndex);
		LinearConstraints.ConstraintStates.RemoveAtSwap(LinearConstraintIndex);
		LinearConstraints.ConstraintSettings.RemoveAtSwap(LinearConstraintIndex);
		LinearConstraints.bJointsDirty = true;
	}

	//This function moves one constraint info from nonlinear constraint into linear constraints
	void FPBDJointCombinedConstraints::ConvertNonLinearConstraintToLinear(const int32 NonLinearConstraintIndex)
	{
		if (!ensureMsgf(NonLinearConstraintIndex > INDEX_NONE && NonLinearConstraintIndex < NonLinearConstraints.NumConstraints(), TEXT("NonLinear constraint accessed is out of bounds!")))
		{
			return; 
		}

		LinearConstraints.bJointsDirty = true;
		int LinearConstraintsNum = LinearConstraints.Handles.Num();
		LinearConstraints.Handles.Add(MoveTemp(NonLinearConstraints.Handles[NonLinearConstraintIndex]));
		LinearConstraints.ConstraintParticles.Add(MoveTemp(NonLinearConstraints.ConstraintParticles[NonLinearConstraintIndex]));
		LinearConstraints.ConstraintStates.Add(MoveTemp(NonLinearConstraints.ConstraintStates[NonLinearConstraintIndex]));
		LinearConstraints.ConstraintSettings.Add(MoveTemp(NonLinearConstraints.ConstraintSettings[NonLinearConstraintIndex]));
		LinearConstraints.ConstraintSettings[LinearConstraintsNum].bUseLinearSolver = true;
		LinearConstraints.Handles[LinearConstraintsNum]->SetConstraintIndex(LinearConstraintsNum);
		LinearConstraints.Handles[LinearConstraintsNum]->SetConstraintContainer(&LinearConstraints);

		NonLinearConstraints.Handles.RemoveAtSwap(NonLinearConstraintIndex);
		NonLinearConstraints.ConstraintParticles.RemoveAtSwap(NonLinearConstraintIndex);
		NonLinearConstraints.ConstraintStates.RemoveAtSwap(NonLinearConstraintIndex);
		NonLinearConstraints.ConstraintSettings.RemoveAtSwap(NonLinearConstraintIndex);
		NonLinearConstraints.bJointsDirty = true;
	}

	void FPBDJointCombinedConstraints::MergeLinearConstraintsIntoNonLinearConstraints()
	{
		while (LinearConstraints.NumConstraints() > 0)
		{
			ConvertLinearConstraintToNonLinear(0);
		}
	}

	void FPBDJointCombinedConstraints::MergeNonLinearConstraintsIntoLinearConstraints()
	{
		while (NonLinearConstraints.NumConstraints() > 0)
		{
			ConvertNonLinearConstraintToLinear(0);
		}
	}


}

namespace Chaos
{
	template class TIndexedContainerConstraintHandle<FPBDJointConstraints>;
}
