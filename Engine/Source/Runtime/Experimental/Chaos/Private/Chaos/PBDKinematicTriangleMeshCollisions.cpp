// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/PBDKinematicTriangleMeshCollisions.h"
#include "Chaos/Plane.h"
#include "Chaos/Triangle.h"
#include "Chaos/TriangleCollisionPoint.h"
#include "Chaos/TriangleMesh.h"
#include "Chaos/PBDSoftsSolverParticles.h"
#include "Chaos/SoftsSolverParticlesRange.h"
#if INTEL_ISPC
#include "PBDKinematicTriangleMeshCollisions.ispc.generated.h"
#endif

#if !COMPILE_WITHOUT_UNREAL_SUPPORT
#include "Chaos/Framework/Parallel.h"
#include <atomic>

#if INTEL_ISPC && !UE_BUILD_SHIPPING
static_assert(sizeof(ispc::FVector3f) == sizeof(Chaos::Softs::FSolverVec3), "sizeof(ispc::FVector3f) != sizeof(Chaos::Softs::FSolverVec3)");
static_assert(sizeof(ispc::FVector4f) == sizeof(Chaos::Softs::FPAndInvM), "sizeof(ispc::FVector4f) != sizeof(Chaos::Softs::FPAndInvM");
static_assert(sizeof(ispc::FVector4f) == sizeof(Chaos::Softs::FSolverRotation3), "sizeof(ispc::FVector4f) != sizeof(Chaos::Softs::FSolverRotation3");
static_assert(sizeof(ispc::FIntVector) == sizeof(Chaos::TVec3<int32>), "sizeof(ispc::FIntVector) != sizeof(Chaos::TVec3<int32>");

bool bChaos_KinematicTriangleMesh_ISPC_Enabled = CHAOS_KINEMATIC_TRIANGLE_COLLISIONS_ISPC_ENABLED_DEFAULT;
FAutoConsoleVariableRef CVarChaosKinematicTriangleMeshCollisionsISPCEnabled(TEXT("p.Chaos.KinematicTriangleMeshCollisions.ISPC"), bChaos_KinematicTriangleMesh_ISPC_Enabled, TEXT("Whether to use ISPC optimizations in kinematic triangle mesh collision constraints"));
#endif

#if UE_BUILD_SHIPPING
static constexpr Chaos::Softs::FSolverReal KinematicColliderMaxTimer = (Chaos::Softs::FSolverReal)0.1f;
static constexpr Chaos::Softs::FSolverReal KinematicColliderFalloffMultiplier = (Chaos::Softs::FSolverReal)1.f;
static constexpr Chaos::Softs::FSolverReal KinematicColliderMaxDepthMultiplier = (Chaos::Softs::FSolverReal)10.f;
#else
Chaos::Softs::FSolverReal KinematicColliderMaxTimer = (Chaos::Softs::FSolverReal)0.1f; 
FAutoConsoleVariableRef CVarChaosKinematicTriangleMeshCollisionsMaxTimer(TEXT("p.Chaos.KinematicTriangleMeshCollisions.MaxTimer"), KinematicColliderMaxTimer, TEXT("Amount of time (in seconds) to remember a kinematic collision connection after it has moved more than Thickness away. Increasing this can reduce jitter at the cost of more computation."));

Chaos::Softs::FSolverReal KinematicColliderFalloffMultiplier = (Chaos::Softs::FSolverReal)1.f;
FAutoConsoleVariableRef CVarChaosKinematicTriangleMeshCollisionsFalloffMultiplier(TEXT("p.Chaos.KinematicTriangleMeshCollisions.FalloffMultiplier"), KinematicColliderFalloffMultiplier, TEXT("Tangential distance away from a triangle (scaled by thickness) beyond which a point isn't considered to be kinematically colliding"));

Chaos::Softs::FSolverReal KinematicColliderMaxDepthMultiplier = (Chaos::Softs::FSolverReal)10.f;
FAutoConsoleVariableRef CVarChaosKinematicTriangleMeshCollisionsaxDepthMultiplier(TEXT("p.Chaos.KinematicTriangleMeshCollisions.MaxDepthMultiplier"), KinematicColliderMaxDepthMultiplier, TEXT("Penetration depth beyond which we ignore the kinematic collision (so you don't push through the wrong side)"));
#endif

namespace Chaos::Softs {

	void FPBDKinematicTriangleMeshCollisions::Init(const FSolverParticlesRange& Particles, const FSolverReal Dt)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPBDKinematicTriangleMeshCollisions_Init);
		check(Particles.Size() == NumParticles);

		if (!TriangleMesh || !SpatialHash || TriangleMesh->GetNumElements() == 0)
		{
			Reset();
			return;
		}

		if (Timers.Num() != NumParticles)
		{
			Timers.Init(TMap<int32, FSolverReal>(), NumParticles);
		}

		CollidingParticles.SetNumUninitialized(NumParticles);
		CollidingElements.SetNumUninitialized(NumParticles);

		std::atomic<int32> KinematicConstraintIndex(0);
		PhysicsParallelFor(NumParticles,
			[this, Dt, &Particles, &KinematicConstraintIndex](int32 Index)
			{
				if (Particles.InvM(Index) == (FSolverReal)0.)
				{
					return;
				}
				constexpr FSolverReal ExtraThicknessMult = 1.5f;
				const FSolverReal ParticleThickness = Thickness.GetValue(Index);

				// Increment existing timers and remove any elements that are too old.
				for (TMap<int32, FSolverReal>::TIterator TimerIter = Timers[Index].CreateIterator(); TimerIter; ++TimerIter)
				{
					TimerIter.Value() += Dt;
					if (TimerIter.Value() > KinematicColliderMaxTimer)
					{
						TimerIter.RemoveCurrent();
					}
				}

				const FSolverVec3& MeshSpacePosition = Particles.X(Index);

				TArray< TTriangleCollisionPoint<FSolverReal> > KinematicResult;
				if (TriangleMesh->PointProximityQuery(*SpatialHash, Positions, Index, MeshSpacePosition, ParticleThickness * ExtraThicknessMult, ColliderThickness * ExtraThicknessMult, [](const int32 PointIndex, const int32 SubMeshTriangleIndex)->bool { return true; }, KinematicResult))
				{
					if (KinematicResult.Num() > MaxKinematicConnectionsPerPoint)
					{
						// TODO: once we have a PartialSort, use that instead here.
						KinematicResult.Sort(
							[](const TTriangleCollisionPoint<FSolverReal>& First, const TTriangleCollisionPoint<FSolverReal>& Second)->bool
							{
								return First.Phi < Second.Phi;
							}
						);
						KinematicResult.SetNum(MaxKinematicConnectionsPerPoint, EAllowShrinking::No);
					}
					for (const TTriangleCollisionPoint<FSolverReal>& CollisionPoint : KinematicResult)
					{
						Timers[Index].FindOrAdd(CollisionPoint.Indices[1], (FSolverReal)0.f);
					}

					// Get MaxKinematicConnectionsPerPoint most recent
					if (Timers[Index].Num() > MaxKinematicConnectionsPerPoint)
					{
						Timers[Index].ValueSort([](const FSolverReal A, const FSolverReal B) { return A < B; });
					}

					if (Timers[Index].Num() > 0)
					{
						const int32 IndexToWrite = KinematicConstraintIndex.fetch_add(1);
						CollidingParticles[IndexToWrite] = Index;

						int32 LocalIndex = 0;
						for (TMap<int32, FSolverReal>::TIterator TimerIter = Timers[Index].CreateIterator(); TimerIter; ++TimerIter)
						{
							if (LocalIndex < MaxKinematicConnectionsPerPoint)
							{
								CollidingElements[IndexToWrite][LocalIndex++] = TimerIter.Key();
							}
							else
							{
								TimerIter.RemoveCurrent();
							}
						}
						for (; LocalIndex < MaxKinematicConnectionsPerPoint; ++LocalIndex)
						{
							CollidingElements[IndexToWrite][LocalIndex] = INDEX_NONE;
						}
					}
				}
			}
		);

		// Shrink the arrays to the actual number of found constraints.
		const int32 KinematicConstraintNum = KinematicConstraintIndex.load();
		CollidingParticles.SetNum(KinematicConstraintNum, EAllowShrinking::No);
		CollidingElements.SetNum(KinematicConstraintNum, EAllowShrinking::No);
	}

	void FPBDKinematicTriangleMeshCollisions::Apply(FSolverParticlesRange& Particles, const FSolverReal Dt) const
	{
		if (!CollidingParticles.Num())
		{
			return;
		}
		check(TriangleMesh);
		TRACE_CPUPROFILER_EVENT_SCOPE(FPBDKinematicTriangleMeshCollisions_ApplyKinematicConstraints);

#if INTEL_ISPC
		static_assert(sizeof(ispc::FIntVector) == sizeof(TVector<int32, MaxKinematicConnectionsPerPoint>), "sizeof(ispc::FIntVector) != sizeof(TVector<int32, MaxKinematicConnectionsPerPoint>)");
		if (bRealTypeCompatibleWithISPC && bChaos_KinematicTriangleMesh_ISPC_Enabled)
		{
			const bool bWithFriction = FrictionCoefficient.HasWeightMap() || (FSolverReal)FrictionCoefficient > (FSolverReal)UE_KINDA_SMALL_NUMBER;

			if (Thickness.HasWeightMap())
			{
				if (bWithFriction)
				{
					if (PAndInvM.IsEmpty())
					{
						ispc::ApplyKinematicTriangleCollisionsWithFrictionAndMaps(
							(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
							(const ispc::FVector3f*)Particles.XArray().GetData(),
							(const ispc::FVector3f*)Positions.GetData(),
							(const ispc::FVector3f*)Velocities.GetData(),
							CollidingParticles.GetData(),
							(const ispc::FIntVector*)CollidingElements.GetData(),
							(const ispc::FIntVector*)TriangleMesh->GetElements().GetData(),
							reinterpret_cast<const ispc::FVector2f&>(Thickness.GetOffsetRange()),
							Thickness.GetMapValues().GetData(),
							Dt,
							ColliderThickness,
							KinematicColliderFalloffMultiplier,
							KinematicColliderMaxDepthMultiplier,
							Stiffness,
							FrictionCoefficient.HasWeightMap(),
							reinterpret_cast<const ispc::FVector2f&>(FrictionCoefficient.GetOffsetRange()),
							FrictionCoefficient.GetMapValues().GetData(),
							CollidingParticles.Num()
						);
					}
					else
					{
						ispc::ApplyKinematicTriangleCollisionsWithFrictionAndMapsPAndInvM(
							(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
							(const ispc::FVector3f*)Particles.XArray().GetData(),
							(const ispc::FVector4f*)PAndInvM.GetData(),
							(const ispc::FVector3f*)Velocities.GetData(),
							CollidingParticles.GetData(),
							(const ispc::FIntVector*)CollidingElements.GetData(),
							(const ispc::FIntVector*)TriangleMesh->GetElements().GetData(),
							reinterpret_cast<const ispc::FVector2f&>(Thickness.GetOffsetRange()),
							Thickness.GetMapValues().GetData(),
							Dt,
							ColliderThickness,
							KinematicColliderFalloffMultiplier,
							KinematicColliderMaxDepthMultiplier,
							Stiffness,
							FrictionCoefficient.HasWeightMap(),
							reinterpret_cast<const ispc::FVector2f&>(FrictionCoefficient.GetOffsetRange()),
							FrictionCoefficient.GetMapValues().GetData(),
							CollidingParticles.Num()
						);
					}
				}
				else
				{
					if (PAndInvM.IsEmpty())
					{
						ispc::ApplyKinematicTriangleCollisionsWithMaps(
							(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
							(const ispc::FVector3f*)Positions.GetData(),
							CollidingParticles.GetData(),
							(const ispc::FIntVector*)CollidingElements.GetData(),
							(const ispc::FIntVector*)TriangleMesh->GetElements().GetData(),
							reinterpret_cast<const ispc::FVector2f&>(Thickness.GetOffsetRange()),
							Thickness.GetMapValues().GetData(),
							ColliderThickness,
							KinematicColliderFalloffMultiplier,
							KinematicColliderMaxDepthMultiplier,
							Stiffness,
							CollidingParticles.Num()
						);
					}
					else
					{
						ispc::ApplyKinematicTriangleCollisionsWithMapsPAndInvM(
							(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
							(const ispc::FVector4f*)PAndInvM.GetData(),
							CollidingParticles.GetData(),
							(const ispc::FIntVector*)CollidingElements.GetData(),
							(const ispc::FIntVector*)TriangleMesh->GetElements().GetData(),
							reinterpret_cast<const ispc::FVector2f&>(Thickness.GetOffsetRange()),
							Thickness.GetMapValues().GetData(),
							ColliderThickness,
							KinematicColliderFalloffMultiplier,
							KinematicColliderMaxDepthMultiplier,
							Stiffness,
							CollidingParticles.Num()
						);
					}
				}
			}
			else
			{
				const FSolverReal Height = (FSolverReal)Thickness + ColliderThickness;
				const FSolverReal OneOverTangentialFalloffDist = (FSolverReal)1.f / FMath::Max(KinematicColliderFalloffMultiplier * Height, UE_KINDA_SMALL_NUMBER);
				const FSolverReal MaxDepth = -Height * KinematicColliderMaxDepthMultiplier;
				if (FrictionCoefficient.HasWeightMap())
				{
					if (PAndInvM.IsEmpty())
					{
						ispc::ApplyKinematicTriangleCollisionsWithFrictionMap(
							(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
							(const ispc::FVector3f*)Particles.XArray().GetData(),
							(const ispc::FVector3f*)Positions.GetData(),
							(const ispc::FVector3f*)Velocities.GetData(),
							CollidingParticles.GetData(),
							(const ispc::FIntVector*)CollidingElements.GetData(),
							(const ispc::FIntVector*)TriangleMesh->GetElements().GetData(),
							Dt,
							Height,
							OneOverTangentialFalloffDist,
							MaxDepth,
							Stiffness,
							reinterpret_cast<const ispc::FVector2f&>(FrictionCoefficient.GetOffsetRange()),
							FrictionCoefficient.GetMapValues().GetData(),
							CollidingParticles.Num()
						);
					}
					else
					{
						ispc::ApplyKinematicTriangleCollisionsWithFrictionMapPAndInvM(
							(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
							(const ispc::FVector3f*)Particles.XArray().GetData(),
							(const ispc::FVector4f*)PAndInvM.GetData(),
							(const ispc::FVector3f*)Velocities.GetData(),
							CollidingParticles.GetData(),
							(const ispc::FIntVector*)CollidingElements.GetData(),
							(const ispc::FIntVector*)TriangleMesh->GetElements().GetData(),
							Dt,
							Height,
							OneOverTangentialFalloffDist,
							MaxDepth,
							Stiffness,
							reinterpret_cast<const ispc::FVector2f&>(FrictionCoefficient.GetOffsetRange()),
							FrictionCoefficient.GetMapValues().GetData(),
							CollidingParticles.Num()
						);
					}
				}
				else if (bWithFriction)
				{
					if (PAndInvM.IsEmpty())
					{
						ispc::ApplyKinematicTriangleCollisionsWithFriction(
							(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
							(const ispc::FVector3f*)Particles.XArray().GetData(),
							(const ispc::FVector3f*)Positions.GetData(),
							(const ispc::FVector3f*)Velocities.GetData(),
							CollidingParticles.GetData(),
							(const ispc::FIntVector*)CollidingElements.GetData(),
							(const ispc::FIntVector*)TriangleMesh->GetElements().GetData(),
							Dt,
							Height,
							OneOverTangentialFalloffDist,
							MaxDepth,
							Stiffness,
							(FSolverReal)FrictionCoefficient,
							CollidingParticles.Num()
						);
					}
					else
					{
						ispc::ApplyKinematicTriangleCollisionsWithFrictionPAndInvM(
							(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
							(const ispc::FVector3f*)Particles.XArray().GetData(),
							(const ispc::FVector4f*)PAndInvM.GetData(),
							(const ispc::FVector3f*)Velocities.GetData(),
							CollidingParticles.GetData(),
							(const ispc::FIntVector*)CollidingElements.GetData(),
							(const ispc::FIntVector*)TriangleMesh->GetElements().GetData(),
							Dt,
							Height,
							OneOverTangentialFalloffDist,
							MaxDepth,
							Stiffness,
							(FSolverReal)FrictionCoefficient,
							CollidingParticles.Num()
						);
					}
				}
				else
				{
					if (PAndInvM.IsEmpty())
					{
						// No friction, no maps
						ispc::ApplyKinematicTriangleCollisions(
							(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
							(const ispc::FVector3f*)Positions.GetData(),
							CollidingParticles.GetData(),
							(const ispc::FIntVector*)CollidingElements.GetData(),
							(const ispc::FIntVector*)TriangleMesh->GetElements().GetData(),
							Height,
							OneOverTangentialFalloffDist,
							MaxDepth,
							Stiffness,
							CollidingParticles.Num()
						);
					}
					else
					{
						// No friction, no maps
						ispc::ApplyKinematicTriangleCollisionsPAndInvM(
							(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
							(const ispc::FVector4f*)PAndInvM.GetData(),
							CollidingParticles.GetData(),
							(const ispc::FIntVector*)CollidingElements.GetData(),
							(const ispc::FIntVector*)TriangleMesh->GetElements().GetData(),
							Height,
							OneOverTangentialFalloffDist,
							MaxDepth,
							Stiffness,
							CollidingParticles.Num()
						);
					}
				}
			}
		}
		else
#endif
		{
			for (int32 Index = 0; Index < CollidingParticles.Num(); ++Index)
			{
				const int32 Index1 = CollidingParticles[Index];
				const FSolverReal ParticleThickness = Thickness.GetValue(Index1);

				const FSolverReal Height = ParticleThickness + ColliderThickness;
				const FSolverReal OneOverTangentialFalloffDist = (FSolverReal)1.f / FMath::Max(KinematicColliderFalloffMultiplier * Height, UE_KINDA_SMALL_NUMBER);
				const FSolverReal MaxDepth = -Height * KinematicColliderMaxDepthMultiplier;

				for (int32 EIndex = 0; EIndex < MaxKinematicConnectionsPerPoint; ++EIndex)
				{
					const int32 ElemIndex = CollidingElements[Index][EIndex];
					if (ElemIndex == INDEX_NONE)
					{
						break;
					}

					const int32 Index2 = TriangleMesh->GetElements()[ElemIndex][0];
					const int32 Index3 = TriangleMesh->GetElements()[ElemIndex][1];
					const int32 Index4 = TriangleMesh->GetElements()[ElemIndex][2];

					FSolverVec3& P1 = Particles.P(Index1);
					const FSolverVec3& P2 = Positions[Index2];
					const FSolverVec3& P3 = Positions[Index3];
					const FSolverVec3& P4 = Positions[Index4];

					const TTriangle<FSolverReal> Triangle(P2, P3, P4);
					const FSolverVec3  Normal = -Triangle.GetNormal(); // normals are flipped from UE

					FSolverVec3 Bary;
					const FSolverVec3 P = FindClosestPointAndBaryOnTriangle(P2, P3, P4, P1, Bary);
					const FSolverVec3 Difference = P1 - P;
					const FSolverReal NormalDifference = Difference.Dot(Normal);

					if (NormalDifference >= Height || NormalDifference < MaxDepth)
					{
						continue;
					}

					const FSolverReal TangentialDifference = (Difference - NormalDifference * Normal).Size();
					const FSolverReal TangentialFalloff = (FSolverReal)1.f - TangentialDifference * OneOverTangentialFalloffDist;
					if (TangentialFalloff <= 0.f)
					{
						continue;
					}

					const FSolverReal NormalDelta = Height - NormalDifference;
					const FSolverVec3 RepulsionDelta = Stiffness * TangentialFalloff * NormalDelta * Normal;

					P1 += RepulsionDelta;

					const FSolverReal KinematicFrictionCoefficient = FrictionCoefficient.GetValue(Index1);
					if (KinematicFrictionCoefficient > 0)
					{
						const FSolverVec3& X1 = Particles.X(Index1);

						const FSolverVec3 V = Bary[0] * Velocities[Index2] + Bary[1] * Velocities[Index3] + Bary[2] * Velocities[Index4];

						const FSolverVec3 RelativeDisplacement = (P1 - X1) - V * Dt;
						const FSolverVec3 RelativeDisplacementTangent = RelativeDisplacement - RelativeDisplacement.Dot(Normal) * Normal;
						const FSolverReal RelativeDisplacementTangentLength = RelativeDisplacementTangent.Length();
						const FSolverReal PositionCorrection = FMath::Min(NormalDelta * KinematicFrictionCoefficient, RelativeDisplacementTangentLength);
						const FSolverReal CorrectionRatio = RelativeDisplacementTangentLength < UE_SMALL_NUMBER ? 0.f : PositionCorrection / RelativeDisplacementTangentLength;
						const FSolverVec3 FrictionDelta = -CorrectionRatio * RelativeDisplacementTangent;

						P1 += FrictionDelta;
					}
				}
			}
		}
	}

}  // End namespace Chaos::Softs

#endif
