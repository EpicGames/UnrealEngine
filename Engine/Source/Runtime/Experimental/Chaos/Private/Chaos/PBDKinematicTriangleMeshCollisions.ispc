// Copyright Epic Games, Inc. All Rights Reserved.

#define EXPLICIT_VECTOR4 1

#include "Math/Vector.isph"
#include "Math/Transform.isph"
#include "Chaos/PBDSofts.isph"

static inline FVector3f ProjectToP01(const FVector3f& P0, const FVector3f& P1, float Size10, float ProjP1, FVector3f& Bary)
{
	Bary.V[2] = 0.f;
	Bary.V[1] = clamp(ProjP1 / Size10, 0.f, 1.f);
	Bary.V[0] = 1.f - Bary.V[1];
	return Bary.V[0] * P0 + Bary.V[1] * P1;
}

static inline FVector3f ProjectToP02(const FVector3f& P0, const FVector3f& P2, float Size20, float ProjP2, FVector3f& Bary)
{
	Bary.V[1] = 0.f;
	Bary.V[2] = clamp(ProjP2 / Size20, 0.f, 1.f);
	Bary.V[0] = 1.f - Bary.V[2];
	return Bary.V[0] * P0 + Bary.V[2] * P2;
}

static inline FVector3f ProjectToP12(const FVector3f& P1, const FVector3f& P2, const FVector3f& P, FVector3f& Bary)
{
	const FVector3f P2P1 = P2 - P1;

	Bary.V[0] = 0.f;
	Bary.V[2] = clamp(VectorDot(P2P1, P - P1) / VectorSizeSquared(P2P1), 0.f, 1.f);
	Bary.V[1] = 1.f - Bary.V[2];
	return Bary.V[1] * P1 + Bary.V[2] * P2;
}

static inline FVector3f GetTriangleNormalFlipped(const FVector3f&P0, const FVector3f& P1, const FVector3f& P2)
{
	// This will return -Chaos::TTriangle<float>::GetNormal.
	return VectorGetSafeNormal(VectorCross(P2 - P0, P1 - P0), FloatZeroVector);
}

static inline FVector3f FindClosestPointAndBaryOnTriangle(const FVector3f& P0, const FVector3f& P1, const FVector3f& P2, const FVector3f& P, FVector3f& Bary)
{
	const FVector3f P10 = P1 - P0;
	const FVector3f P20 = P2 - P0;
	const FVector3f P21 = P2 - P1;
	const FVector3f PP0 = P - P0;

	const float Size10 = VectorSizeSquared(P10);
	const float Size20 = VectorSizeSquared(P20);
	const float Size21 = VectorSizeSquared(P21);

	const float ProjP1 = VectorDot(PP0, P10);
	const float ProjP2 = VectorDot(PP0, P20);

	
	// Degenerate triangles
	if (Size10 < FLOAT_SMALL_NUMBER)
	{
		if (Size20 < FLOAT_SMALL_NUMBER)
		{
			// Triangle is (nearly) a single point.
			Bary.V[0] = 1.f;
			Bary.V[1] = Bary.V[2] = 0.f;
			return P0;
		}

		// Triangle is a line segment from P0(=P1) to P2. Project to that line segment.
		return ProjectToP02(P0, P2, Size20, ProjP2, Bary);
	}
	if (Size20 < FLOAT_SMALL_NUMBER)
	{
		// Triangle is a line segment from P0(=P2) to P1. Project to that line segment.
		return ProjectToP01(P0, P1, Size10, ProjP1, Bary);
	}


	const float ProjSides = VectorDot(P10, P20);
	const float Denom = Size10 * Size20 - ProjSides * ProjSides;

	if (Denom < FLOAT_SMALL_NUMBER)
	{
		// Triangle is a line segment from P0 to P1(=P2), or otherwise the 3 points are (nearly) colinear. Project to the longest edge.
		if (Size21 > Size20)
		{
			return ProjectToP12(P1, P2, P, Bary);
		}
		else if (Size20 > Size10)
		{
			return ProjectToP02(P2, P2, Size20, ProjP2, Bary);
		}
		else
		{
			return ProjectToP01(P0, P1, Size10, ProjP1, Bary);
		}
	}

	// Non-degenerate triangle

	// Numerators of barycentric coordinates if P is inside triangle
	const float BaryYNum = (Size20 * ProjP1 - ProjSides * ProjP2);
	const float BaryZNum = (Size10 * ProjP2 - ProjSides * ProjP1);

	// Using this specific epsilon to have parity with FindClosestPointOnTriangle
	const uniform float Epsilon = 1e-4f;
	const float EpsilonScaledByDenom = Epsilon * Denom;

	if (BaryYNum + BaryZNum > Denom * (1 + Epsilon)) // i.e., Bary.X < -Epsilon.
	{
		if (BaryYNum < -EpsilonScaledByDenom)
		{
			// Bary.X < 0, Bary.Y < 0, (Bary.Z > 1)

			// We're outside the triangle in the region opposite angle P2. (if angle at P2 is > 90 degrees, closest point could be on one of these segments)
			// Should project onto line segment P02 or P12
			if (ProjP2 < Size20)
			{
				// Closer to P02 
				// Project to line segment from P02
				return ProjectToP02(P0, P2, Size20, ProjP2, Bary);
			}
		} 
		else if (BaryZNum < -EpsilonScaledByDenom)
		{
			// Bary.X < 0, Bary.Y > 1, Bary.Z < 0
			
			// We're outside the triangle in the region opposite the angle P1.
			// Should project onto line segment P01 or P12
			if (ProjP1 < Size10)
			{
				// Closer to P01.
				// Project to line segment from P01
				return ProjectToP01(P0, P1, Size10, ProjP1, Bary);
			}
		}

		// Bary.X < 0, Bary.Y >=0, Bary.Z >= 0

		// We're outside the triangle in the region closest to P12
		// Project to line segment from P12
		return ProjectToP12(P1, P2, P, Bary);
	}
	else if (BaryYNum < -EpsilonScaledByDenom)
	{
		if (BaryZNum < -EpsilonScaledByDenom)
		{
			// Bary.X > 1, Bary.Y < 0, Bary.Z < 0

			// We're outside the triangle in the region opposite P0.
			// Should project onto line segment P01 or P02 
			if (ProjP1 > 0.f)
			{
				// Closer to P01.
				// Project to line segment from P01
				return ProjectToP01(P0, P1, Size10, ProjP1, Bary);
			}
		}

		// Bary.X >= 0, Bary.Y < 0, Bary.Z >= 0
		// We're outside the triangle in region closest to P02.
		// Project to line segment from P02
		return ProjectToP02(P0, P2, Size20, ProjP2, Bary);
	}
	else if (BaryZNum < -EpsilonScaledByDenom)
	{
		// Bary.X >=0, Bary.Y >=0, Bary.Z < 0
		// We're outside the triangle in the region closest to P01
		// Project to line segment from P0 to P1
		return ProjectToP01(P0, P1, Size10, ProjP1, Bary);
	}
	
	// Bary.X >= 0, Bary.Y >= 0, Bary.Z >= 0
	// Interior of triangle!
	Bary.V[1] = BaryYNum / Denom;
	Bary.V[2] = BaryZNum / Denom;
	Bary.V[0] = 1.f - Bary.V[1] - Bary.V[2];
	return Bary.V[0] * P0 + Bary.V[1] * P1 + Bary.V[2] * P2;
}

export void ApplyKinematicTriangleCollisions(
			uniform FVector4f PandInvM[],
			const uniform FVector3f KinematicPositions[],
			const uniform int32 KinematicCollidingParticles[],
			const uniform FIntVector KinematicColliderElements[],
			const uniform FIntVector TriangleMeshElements[],
			const uniform float Height,
			const uniform float OneOverTangentialFalloffDist,
			const uniform float MaxDepth,
			const uniform float KinematicColliderStiffness,
			const uniform int32 NumConstraints)
{
	foreach(Index = 0...NumConstraints)
	{
		const int32 Index1 = KinematicCollidingParticles[Index];

		const FVector4f PandInvM1 = VectorGather(&PandInvM[Index1]);
		varying FVector3f P1;
		varying float IM;
		UnzipPandInvM(PandInvM1, P1, IM);

		const varying FIntVector Elements = VectorLoad(&KinematicColliderElements[extract(Index,0)]);

		for(uniform int32 EIndex = 0; EIndex < 3; ++EIndex)
		{
			const varying int32 ElementIndex = Elements.V[EIndex];
			if(ElementIndex == -1)
			{
				break;
			}

			const FIntVector Element = VectorGather(&TriangleMeshElements[ElementIndex]);

			const varying int32 Index2 = Element.V[0];
			const varying int32 Index3 = Element.V[1];
			const varying int32 Index4 = Element.V[2];
		
			const varying FVector3f P2 = VectorGather(&KinematicPositions[Index2]);
			const varying FVector3f P3 = VectorGather(&KinematicPositions[Index3]);
			const varying FVector3f P4 = VectorGather(&KinematicPositions[Index4]);

			const varying FVector3f Normal = GetTriangleNormalFlipped(P2, P3, P4);

			varying FVector3f Bary;
			const varying FVector3f P = FindClosestPointAndBaryOnTriangle(P2, P3, P4, P1, Bary);
			const varying FVector3f Difference = P1 - P;
			const varying float NormalDifference = VectorDot(Difference, Normal);
			if(NormalDifference >= Height || NormalDifference < MaxDepth)
			{
				continue;
			}

			const varying float TangentialDifference = VectorSize(Difference - NormalDifference * Normal);
			const uniform float TangentialFalloffMultiplier = 1.5f;
			const varying float TangentialFalloff = 1.f - TangentialDifference * OneOverTangentialFalloffDist;
			if(TangentialFalloff <= 0.f)
			{
				continue;
			}

			const float NormalDelta = Height - NormalDifference;
			const FVector3f RepulsionDelta = KinematicColliderStiffness * TangentialFalloff * NormalDelta *  Normal;

			P1 = P1 + RepulsionDelta;
		}

		VectorScatter(&PandInvM[Index1], SetVector4(P1, IM));
	}
}

export void ApplyKinematicTriangleCollisionsWithFriction(
			uniform FVector4f PandInvM[],
			const uniform FVector3f ParticlesX[],
			const uniform FVector3f KinematicPositions[],
			const uniform FVector3f KinematicVelocities[],
			const uniform int32 KinematicCollidingParticles[],
			const uniform FIntVector KinematicColliderElements[],
			const uniform FIntVector TriangleMeshElements[],
			const uniform float Dt,
			const uniform float Height,
			const uniform float OneOverTangentialFalloffDist,
			const uniform float MaxDepth,
			const uniform float KinematicColliderStiffness, 
			const uniform float KinematicColliderFrictionCoefficient, 
			const uniform int32 NumConstraints)
{
	foreach(Index = 0...NumConstraints)
	{
		const int32 Index1 = KinematicCollidingParticles[Index];

		const FVector4f PandInvM1 = VectorGather(&PandInvM[Index1]);
		varying FVector3f P1;
		varying float IM;
		UnzipPandInvM(PandInvM1, P1, IM);

		const varying FIntVector Elements = VectorLoad(&KinematicColliderElements[extract(Index,0)]);

		for(uniform int32 EIndex = 0; EIndex < 3; ++EIndex)
		{
			const varying int32 ElementIndex = Elements.V[EIndex];
			if(ElementIndex == -1)
			{
				break;
			}

			const FIntVector Element = VectorGather(&TriangleMeshElements[ElementIndex]);

			const varying int32 Index2 = Element.V[0];
			const varying int32 Index3 = Element.V[1];
			const varying int32 Index4 = Element.V[2];
		
			const varying FVector3f P2 = VectorGather(&KinematicPositions[Index2]);
			const varying FVector3f P3 = VectorGather(&KinematicPositions[Index3]);
			const varying FVector3f P4 = VectorGather(&KinematicPositions[Index4]);

			const varying FVector3f Normal = GetTriangleNormalFlipped(P2, P3, P4);

			varying FVector3f Bary;
			const varying FVector3f P = FindClosestPointAndBaryOnTriangle(P2, P3, P4, P1, Bary);
			const varying FVector3f Difference = P1 - P;
			const varying float NormalDifference = VectorDot(Difference, Normal);
			if(NormalDifference >= Height || NormalDifference < MaxDepth)
			{
				continue;
			}

			const varying float TangentialDifference = VectorSize(Difference - NormalDifference * Normal);
			const uniform float TangentialFalloffMultiplier = 1.5f;
			const varying float TangentialFalloff = 1.f - TangentialDifference * OneOverTangentialFalloffDist;
			if(TangentialFalloff <= 0.f)
			{
				continue;
			}

			const float NormalDelta = Height - NormalDifference;
			const FVector3f RepulsionDelta = KinematicColliderStiffness * TangentialFalloff * NormalDelta *  Normal;

			P1 = P1 + RepulsionDelta;

			const FVector3f X1 = VectorGather(&ParticlesX[Index1]);
			const FVector3f V2 = VectorGather(&KinematicVelocities[Index2]);
			const FVector3f V3 = VectorGather(&KinematicVelocities[Index3]);
			const FVector3f V4 = VectorGather(&KinematicVelocities[Index4]);
			const FVector3f V = Bary.V[0] * V2 + Bary.V[1] * V3 + Bary.V[2] * V4;
			const FVector3f RelativeDisplacement = (P1 - X1) - V * Dt;
			const FVector3f RelativeDisplacementTangent = RelativeDisplacement - VectorDot(RelativeDisplacement,Normal) * Normal;
			const float RelativeDisplacementTangentLength = VectorSize(RelativeDisplacementTangent);
			const float PositionCorrection = min(NormalDelta * KinematicColliderFrictionCoefficient, RelativeDisplacementTangentLength);
			const float CorrectionRatio = RelativeDisplacementTangentLength < FLOAT_SMALL_NUMBER ? 0.f : PositionCorrection / RelativeDisplacementTangentLength;
			const FVector3f FrictionDelta = -CorrectionRatio * RelativeDisplacementTangent;

			P1 = P1 + FrictionDelta;
		}

		VectorScatter(&PandInvM[Index1], SetVector4(P1, IM));
	}
}

export void ApplyKinematicTriangleCollisionsWithFrictionMap(
			uniform FVector4f PandInvM[],
			const uniform FVector3f ParticlesX[],
			const uniform FVector3f KinematicPositions[],
			const uniform FVector3f KinematicVelocities[],
			const uniform int32 KinematicCollidingParticles[],
			const uniform FIntVector KinematicColliderElements[],
			const uniform FIntVector TriangleMeshElements[],
			const uniform float Dt,
			const uniform float Height,
			const uniform float OneOverTangentialFalloffDist,
			const uniform float MaxDepth,
			const uniform float KinematicColliderStiffness, 
			const uniform FVector2f& KinematicColliderFrictionCoefficientOffsetRange, 
			const uniform float KinematicColliderFrictionCoefficientMapValues[],
			const uniform int32 NumConstraints)
{
	foreach(Index = 0...NumConstraints)
	{
		const int32 Index1 = KinematicCollidingParticles[Index];

		const FVector4f PandInvM1 = VectorGather(&PandInvM[Index1]);
		varying FVector3f P1;
		varying float IM;
		UnzipPandInvM(PandInvM1, P1, IM);

		const varying FIntVector Elements = VectorLoad(&KinematicColliderElements[extract(Index,0)]);
		
		#pragma ignore warning(perf)
		const varying float FrictionMapValue = KinematicColliderFrictionCoefficientMapValues[Index1];
		const varying float FrictionCoefficient = KinematicColliderFrictionCoefficientOffsetRange.V[0] + FrictionMapValue * KinematicColliderFrictionCoefficientOffsetRange.V[1];

		for(uniform int32 EIndex = 0; EIndex < 3; ++EIndex)
		{
			const varying int32 ElementIndex = Elements.V[EIndex];
			if(ElementIndex == -1)
			{
				break;
			}

			const FIntVector Element = VectorGather(&TriangleMeshElements[ElementIndex]);

			const varying int32 Index2 = Element.V[0];
			const varying int32 Index3 = Element.V[1];
			const varying int32 Index4 = Element.V[2];
		
			const varying FVector3f P2 = VectorGather(&KinematicPositions[Index2]);
			const varying FVector3f P3 = VectorGather(&KinematicPositions[Index3]);
			const varying FVector3f P4 = VectorGather(&KinematicPositions[Index4]);

			const varying FVector3f Normal = GetTriangleNormalFlipped(P2, P3, P4);

			varying FVector3f Bary;
			const varying FVector3f P = FindClosestPointAndBaryOnTriangle(P2, P3, P4, P1, Bary);
			const varying FVector3f Difference = P1 - P;
			const varying float NormalDifference = VectorDot(Difference, Normal);
			if(NormalDifference >= Height || NormalDifference < MaxDepth)
			{
				continue;
			}

			const varying float TangentialDifference = VectorSize(Difference - NormalDifference * Normal);
			const uniform float TangentialFalloffMultiplier = 1.5f;
			const varying float TangentialFalloff = 1.f - TangentialDifference * OneOverTangentialFalloffDist;
			if(TangentialFalloff <= 0.f)
			{
				continue;
			}

			const float NormalDelta = Height - NormalDifference;
			const FVector3f RepulsionDelta = KinematicColliderStiffness * TangentialFalloff * NormalDelta *  Normal;

			P1 = P1 + RepulsionDelta;

			if (FrictionCoefficient > 0)
			{
				const FVector3f X1 = VectorGather(&ParticlesX[Index1]);
				const FVector3f V2 = VectorGather(&KinematicVelocities[Index2]);
				const FVector3f V3 = VectorGather(&KinematicVelocities[Index3]);
				const FVector3f V4 = VectorGather(&KinematicVelocities[Index4]);
				const FVector3f V = Bary.V[0] * V2 + Bary.V[1] * V3 + Bary.V[2] * V4;
				const FVector3f RelativeDisplacement = (P1 - X1) - V * Dt;
				const FVector3f RelativeDisplacementTangent = RelativeDisplacement - VectorDot(RelativeDisplacement,Normal) * Normal;
				const float RelativeDisplacementTangentLength = VectorSize(RelativeDisplacementTangent);
				const float PositionCorrection = min(NormalDelta * FrictionCoefficient, RelativeDisplacementTangentLength);
				const float CorrectionRatio = RelativeDisplacementTangentLength < FLOAT_SMALL_NUMBER ? 0.f : PositionCorrection / RelativeDisplacementTangentLength;
				const FVector3f FrictionDelta = -CorrectionRatio * RelativeDisplacementTangent;

				P1 = P1 + FrictionDelta;
			}
		}

		VectorScatter(&PandInvM[Index1], SetVector4(P1, IM));
	}
}

export void ApplyKinematicTriangleCollisionsWithMaps(
			uniform FVector4f PandInvM[],
			const uniform FVector3f KinematicPositions[],
			const uniform int32 KinematicCollidingParticles[],
			const uniform FIntVector KinematicColliderElements[],
			const uniform FIntVector TriangleMeshElements[],
			const uniform FVector2f& ParticleThicknessOffsetRange,
			const uniform float ParticleThicknessMapValues[],
			const uniform float KinematicColliderThickness,
			const uniform float KinematicColliderFalloffMultiplier,
			const uniform float KinematicColliderMaxDepthMultiplier,
			const uniform float KinematicColliderStiffness,
			const uniform int32 NumConstraints)
{
	foreach(Index = 0...NumConstraints)
	{
		const int32 Index1 = KinematicCollidingParticles[Index];

		const FVector4f PandInvM1 = VectorGather(&PandInvM[Index1]);
		varying FVector3f P1;
		varying float IM;
		UnzipPandInvM(PandInvM1, P1, IM);

		const varying FIntVector Elements = VectorLoad(&KinematicColliderElements[extract(Index,0)]);
		
		#pragma ignore warning(perf)
		const varying float ThicknessMapValue = ParticleThicknessMapValues[Index1];
		const varying float ParticleThickness = ParticleThicknessOffsetRange.V[0] + ThicknessMapValue * ParticleThicknessOffsetRange.V[1];
		const float Height = ParticleThickness + KinematicColliderThickness;
		const float OneOverTangentialFalloffDist = 1.f / max(KinematicColliderFalloffMultiplier * Height, FLOAT_KINDA_SMALL_NUMBER);
		const float MaxDepth = -Height * KinematicColliderMaxDepthMultiplier;

		for(uniform int32 EIndex = 0; EIndex < 3; ++EIndex)
		{
			const varying int32 ElementIndex = Elements.V[EIndex];
			if(ElementIndex == -1)
			{
				break;
			}

			const FIntVector Element = VectorGather(&TriangleMeshElements[ElementIndex]);

			const varying int32 Index2 = Element.V[0];
			const varying int32 Index3 = Element.V[1];
			const varying int32 Index4 = Element.V[2];
		
			const varying FVector3f P2 = VectorGather(&KinematicPositions[Index2]);
			const varying FVector3f P3 = VectorGather(&KinematicPositions[Index3]);
			const varying FVector3f P4 = VectorGather(&KinematicPositions[Index4]);

			const varying FVector3f Normal = GetTriangleNormalFlipped(P2, P3, P4);

			varying FVector3f Bary;
			const varying FVector3f P = FindClosestPointAndBaryOnTriangle(P2, P3, P4, P1, Bary);
			const varying FVector3f Difference = P1 - P;
			const varying float NormalDifference = VectorDot(Difference, Normal);
			if(NormalDifference >= Height || NormalDifference < MaxDepth)
			{
				continue;
			}

			const varying float TangentialDifference = VectorSize(Difference - NormalDifference * Normal);
			const uniform float TangentialFalloffMultiplier = 1.5f;
			const varying float TangentialFalloff = 1.f - TangentialDifference * OneOverTangentialFalloffDist;
			if(TangentialFalloff <= 0.f)
			{
				continue;
			}

			const float NormalDelta = Height - NormalDifference;
			const FVector3f RepulsionDelta = KinematicColliderStiffness * TangentialFalloff * NormalDelta *  Normal;

			P1 = P1 + RepulsionDelta;
		}

		VectorScatter(&PandInvM[Index1], SetVector4(P1, IM));
	}
}

export void ApplyKinematicTriangleCollisionsWithFrictionAndMaps(
			uniform FVector4f PandInvM[],
			const uniform FVector3f ParticlesX[],
			const uniform FVector3f KinematicPositions[],
			const uniform FVector3f KinematicVelocities[],
			const uniform int32 KinematicCollidingParticles[],
			const uniform FIntVector KinematicColliderElements[],
			const uniform FIntVector TriangleMeshElements[],
			const uniform FVector2f& ParticleThicknessOffsetRange,
			const uniform float ParticleThicknessMapValues[],
			const uniform float Dt,
			const uniform float KinematicColliderThickness,
			const uniform float KinematicColliderFalloffMultiplier,
			const uniform float KinematicColliderMaxDepthMultiplier,
			const uniform float KinematicColliderStiffness, 
			const uniform bool bKinematicColliderFrictionHasMap,
			const uniform FVector2f& KinematicColliderFrictionCoefficientOffsetRange, 
			const uniform float KinematicColliderFrictionCoefficientMapValues[],
			const uniform int32 NumConstraints)
{
	foreach(Index = 0...NumConstraints)
	{
		const int32 Index1 = KinematicCollidingParticles[Index];

		const FVector4f PandInvM1 = VectorGather(&PandInvM[Index1]);
		varying FVector3f P1;
		varying float IM;
		UnzipPandInvM(PandInvM1, P1, IM);

		const varying FIntVector Elements = VectorLoad(&KinematicColliderElements[extract(Index,0)]);
		
		#pragma ignore warning(perf)
		const varying float ThicknessMapValue = ParticleThicknessMapValues[Index1];
		const varying float ParticleThickness = ParticleThicknessOffsetRange.V[0] + ThicknessMapValue * ParticleThicknessOffsetRange.V[1];
		const float Height = ParticleThickness + KinematicColliderThickness;
		const float OneOverTangentialFalloffDist = 1.f / max(KinematicColliderFalloffMultiplier * Height, FLOAT_KINDA_SMALL_NUMBER);
		const float MaxDepth = -Height * KinematicColliderMaxDepthMultiplier;

		varying float FrictionCoefficient = KinematicColliderFrictionCoefficientOffsetRange.V[0];
		if(bKinematicColliderFrictionHasMap)
		{
			#pragma ignore warning(perf)
			const varying float FrictionMapValue = KinematicColliderFrictionCoefficientMapValues[Index1];
			FrictionCoefficient += FrictionMapValue * KinematicColliderFrictionCoefficientOffsetRange.V[1];
		}

		for(uniform int32 EIndex = 0; EIndex < 3; ++EIndex)
		{
			const varying int32 ElementIndex = Elements.V[EIndex];
			if(ElementIndex == -1)
			{
				break;
			}

			const FIntVector Element = VectorGather(&TriangleMeshElements[ElementIndex]);

			const varying int32 Index2 = Element.V[0];
			const varying int32 Index3 = Element.V[1];
			const varying int32 Index4 = Element.V[2];
		
			const varying FVector3f P2 = VectorGather(&KinematicPositions[Index2]);
			const varying FVector3f P3 = VectorGather(&KinematicPositions[Index3]);
			const varying FVector3f P4 = VectorGather(&KinematicPositions[Index4]);

			const varying FVector3f Normal = GetTriangleNormalFlipped(P2, P3, P4);

			varying FVector3f Bary;
			const varying FVector3f P = FindClosestPointAndBaryOnTriangle(P2, P3, P4, P1, Bary);
			const varying FVector3f Difference = P1 - P;
			const varying float NormalDifference = VectorDot(Difference, Normal);
			if(NormalDifference >= Height || NormalDifference < MaxDepth)
			{
				continue;
			}

			const varying float TangentialDifference = VectorSize(Difference - NormalDifference * Normal);
			const uniform float TangentialFalloffMultiplier = 1.5f;
			const varying float TangentialFalloff = 1.f - TangentialDifference * OneOverTangentialFalloffDist;
			if(TangentialFalloff <= 0.f)
			{
				continue;
			}

			const float NormalDelta = Height - NormalDifference;
			const FVector3f RepulsionDelta = KinematicColliderStiffness * TangentialFalloff * NormalDelta *  Normal;

			P1 = P1 + RepulsionDelta;


			if (FrictionCoefficient > 0)
			{
				const FVector3f X1 = VectorGather(&ParticlesX[Index1]);
				const FVector3f V2 = VectorGather(&KinematicVelocities[Index2]);
				const FVector3f V3 = VectorGather(&KinematicVelocities[Index3]);
				const FVector3f V4 = VectorGather(&KinematicVelocities[Index4]);
				const FVector3f V = Bary.V[0] * V2 + Bary.V[1] * V3 + Bary.V[2] * V4;
				const FVector3f RelativeDisplacement = (P1 - X1) - V * Dt;
				const FVector3f RelativeDisplacementTangent = RelativeDisplacement - VectorDot(RelativeDisplacement,Normal) * Normal;
				const float RelativeDisplacementTangentLength = VectorSize(RelativeDisplacementTangent);
				const float PositionCorrection = min(NormalDelta * FrictionCoefficient, RelativeDisplacementTangentLength);
				const float CorrectionRatio = RelativeDisplacementTangentLength < FLOAT_SMALL_NUMBER ? 0.f : PositionCorrection / RelativeDisplacementTangentLength;
				const FVector3f FrictionDelta = -CorrectionRatio * RelativeDisplacementTangent;

				P1 = P1 + FrictionDelta;
			}
		}

		VectorScatter(&PandInvM[Index1], SetVector4(P1, IM));
	}
}


export void ApplyKinematicTriangleCollisionsPAndInvM(
			uniform FVector4f PandInvM[],
			const uniform FVector4f KinematicPositions[],
			const uniform int32 KinematicCollidingParticles[],
			const uniform FIntVector KinematicColliderElements[],
			const uniform FIntVector TriangleMeshElements[],
			const uniform float Height,
			const uniform float OneOverTangentialFalloffDist,
			const uniform float MaxDepth,
			const uniform float KinematicColliderStiffness,
			const uniform int32 NumConstraints)
{
	foreach(Index = 0...NumConstraints)
	{
		const int32 Index1 = KinematicCollidingParticles[Index];

		const FVector4f PandInvM1 = VectorGather(&PandInvM[Index1]);
		varying FVector3f P1;
		varying float IM;
		UnzipPandInvM(PandInvM1, P1, IM);

		const varying FIntVector Elements = VectorLoad(&KinematicColliderElements[extract(Index,0)]);

		for(uniform int32 EIndex = 0; EIndex < 3; ++EIndex)
		{
			const varying int32 ElementIndex = Elements.V[EIndex];
			if(ElementIndex == -1)
			{
				break;
			}

			const FIntVector Element = VectorGather(&TriangleMeshElements[ElementIndex]);

			const varying int32 Index2 = Element.V[0];
			const varying int32 Index3 = Element.V[1];
			const varying int32 Index4 = Element.V[2];
		
			const varying FVector4f PandInvM2 = VectorGather(&KinematicPositions[Index2]);
			const varying FVector4f PandInvM3 = VectorGather(&KinematicPositions[Index3]);
			const varying FVector4f PandInvM4 = VectorGather(&KinematicPositions[Index4]);

			varying FVector3f P2, P3, P4;
			varying float IM2, IM3, IM4;
			UnzipPandInvM(PandInvM2, P2, IM2);
			UnzipPandInvM(PandInvM3, P3, IM3);
			UnzipPandInvM(PandInvM4, P4, IM4);

			const varying FVector3f Normal = GetTriangleNormalFlipped(P2, P3, P4);

			varying FVector3f Bary;
			const varying FVector3f P = FindClosestPointAndBaryOnTriangle(P2, P3, P4, P1, Bary);
			const varying FVector3f Difference = P1 - P;
			const varying float NormalDifference = VectorDot(Difference, Normal);
			if(NormalDifference >= Height || NormalDifference < MaxDepth)
			{
				continue;
			}

			const varying float TangentialDifference = VectorSize(Difference - NormalDifference * Normal);
			const uniform float TangentialFalloffMultiplier = 1.5f;
			const varying float TangentialFalloff = 1.f - TangentialDifference * OneOverTangentialFalloffDist;
			if(TangentialFalloff <= 0.f)
			{
				continue;
			}

			const float NormalDelta = Height - NormalDifference;
			const FVector3f RepulsionDelta = KinematicColliderStiffness * TangentialFalloff * NormalDelta *  Normal;

			P1 = P1 + RepulsionDelta;
		}

		VectorScatter(&PandInvM[Index1], SetVector4(P1, IM));
	}
}

export void ApplyKinematicTriangleCollisionsWithFrictionPAndInvM(
			uniform FVector4f PandInvM[],
			const uniform FVector3f ParticlesX[],
			const uniform FVector4f KinematicPositions[],
			const uniform FVector3f KinematicVelocities[],
			const uniform int32 KinematicCollidingParticles[],
			const uniform FIntVector KinematicColliderElements[],
			const uniform FIntVector TriangleMeshElements[],
			const uniform float Dt,
			const uniform float Height,
			const uniform float OneOverTangentialFalloffDist,
			const uniform float MaxDepth,
			const uniform float KinematicColliderStiffness, 
			const uniform float KinematicColliderFrictionCoefficient, 
			const uniform int32 NumConstraints)
{
	foreach(Index = 0...NumConstraints)
	{
		const int32 Index1 = KinematicCollidingParticles[Index];

		const FVector4f PandInvM1 = VectorGather(&PandInvM[Index1]);
		varying FVector3f P1;
		varying float IM;
		UnzipPandInvM(PandInvM1, P1, IM);

		const varying FIntVector Elements = VectorLoad(&KinematicColliderElements[extract(Index,0)]);

		for(uniform int32 EIndex = 0; EIndex < 3; ++EIndex)
		{
			const varying int32 ElementIndex = Elements.V[EIndex];
			if(ElementIndex == -1)
			{
				break;
			}

			const FIntVector Element = VectorGather(&TriangleMeshElements[ElementIndex]);

			const varying int32 Index2 = Element.V[0];
			const varying int32 Index3 = Element.V[1];
			const varying int32 Index4 = Element.V[2];
		
			const varying FVector4f PandInvM2 = VectorGather(&KinematicPositions[Index2]);
			const varying FVector4f PandInvM3 = VectorGather(&KinematicPositions[Index3]);
			const varying FVector4f PandInvM4 = VectorGather(&KinematicPositions[Index4]);

			varying FVector3f P2, P3, P4;
			varying float IM2, IM3, IM4;
			UnzipPandInvM(PandInvM2, P2, IM2);
			UnzipPandInvM(PandInvM3, P3, IM3);
			UnzipPandInvM(PandInvM4, P4, IM4);

			const varying FVector3f Normal = GetTriangleNormalFlipped(P2, P3, P4);

			varying FVector3f Bary;
			const varying FVector3f P = FindClosestPointAndBaryOnTriangle(P2, P3, P4, P1, Bary);
			const varying FVector3f Difference = P1 - P;
			const varying float NormalDifference = VectorDot(Difference, Normal);
			if(NormalDifference >= Height || NormalDifference < MaxDepth)
			{
				continue;
			}

			const varying float TangentialDifference = VectorSize(Difference - NormalDifference * Normal);
			const uniform float TangentialFalloffMultiplier = 1.5f;
			const varying float TangentialFalloff = 1.f - TangentialDifference * OneOverTangentialFalloffDist;
			if(TangentialFalloff <= 0.f)
			{
				continue;
			}

			const float NormalDelta = Height - NormalDifference;
			const FVector3f RepulsionDelta = KinematicColliderStiffness * TangentialFalloff * NormalDelta *  Normal;

			P1 = P1 + RepulsionDelta;

			const FVector3f X1 = VectorGather(&ParticlesX[Index1]);
			const FVector3f V2 = VectorGather(&KinematicVelocities[Index2]);
			const FVector3f V3 = VectorGather(&KinematicVelocities[Index3]);
			const FVector3f V4 = VectorGather(&KinematicVelocities[Index4]);
			const FVector3f V = Bary.V[0] * V2 + Bary.V[1] * V3 + Bary.V[2] * V4;
			const FVector3f RelativeDisplacement = (P1 - X1) - V * Dt;
			const FVector3f RelativeDisplacementTangent = RelativeDisplacement - VectorDot(RelativeDisplacement,Normal) * Normal;
			const float RelativeDisplacementTangentLength = VectorSize(RelativeDisplacementTangent);
			const float PositionCorrection = min(NormalDelta * KinematicColliderFrictionCoefficient, RelativeDisplacementTangentLength);
			const float CorrectionRatio = RelativeDisplacementTangentLength < FLOAT_SMALL_NUMBER ? 0.f : PositionCorrection / RelativeDisplacementTangentLength;
			const FVector3f FrictionDelta = -CorrectionRatio * RelativeDisplacementTangent;

			P1 = P1 + FrictionDelta;
		}

		VectorScatter(&PandInvM[Index1], SetVector4(P1, IM));
	}
}

export void ApplyKinematicTriangleCollisionsWithFrictionMapPAndInvM(
			uniform FVector4f PandInvM[],
			const uniform FVector3f ParticlesX[],
			const uniform FVector4f KinematicPositions[],
			const uniform FVector3f KinematicVelocities[],
			const uniform int32 KinematicCollidingParticles[],
			const uniform FIntVector KinematicColliderElements[],
			const uniform FIntVector TriangleMeshElements[],
			const uniform float Dt,
			const uniform float Height,
			const uniform float OneOverTangentialFalloffDist,
			const uniform float MaxDepth,
			const uniform float KinematicColliderStiffness, 
			const uniform FVector2f& KinematicColliderFrictionCoefficientOffsetRange, 
			const uniform float KinematicColliderFrictionCoefficientMapValues[],
			const uniform int32 NumConstraints)
{
	foreach(Index = 0...NumConstraints)
	{
		const int32 Index1 = KinematicCollidingParticles[Index];

		const FVector4f PandInvM1 = VectorGather(&PandInvM[Index1]);
		varying FVector3f P1;
		varying float IM;
		UnzipPandInvM(PandInvM1, P1, IM);

		const varying FIntVector Elements = VectorLoad(&KinematicColliderElements[extract(Index,0)]);
		
		#pragma ignore warning(perf)
		const varying float FrictionMapValue = KinematicColliderFrictionCoefficientMapValues[Index1];
		const varying float FrictionCoefficient = KinematicColliderFrictionCoefficientOffsetRange.V[0] + FrictionMapValue * KinematicColliderFrictionCoefficientOffsetRange.V[1];

		for(uniform int32 EIndex = 0; EIndex < 3; ++EIndex)
		{
			const varying int32 ElementIndex = Elements.V[EIndex];
			if(ElementIndex == -1)
			{
				break;
			}

			const FIntVector Element = VectorGather(&TriangleMeshElements[ElementIndex]);

			const varying int32 Index2 = Element.V[0];
			const varying int32 Index3 = Element.V[1];
			const varying int32 Index4 = Element.V[2];
		
			const varying FVector4f PandInvM2 = VectorGather(&KinematicPositions[Index2]);
			const varying FVector4f PandInvM3 = VectorGather(&KinematicPositions[Index3]);
			const varying FVector4f PandInvM4 = VectorGather(&KinematicPositions[Index4]);

			varying FVector3f P2, P3, P4;
			varying float IM2, IM3, IM4;
			UnzipPandInvM(PandInvM2, P2, IM2);
			UnzipPandInvM(PandInvM3, P3, IM3);
			UnzipPandInvM(PandInvM4, P4, IM4);

			const varying FVector3f Normal = GetTriangleNormalFlipped(P2, P3, P4);

			varying FVector3f Bary;
			const varying FVector3f P = FindClosestPointAndBaryOnTriangle(P2, P3, P4, P1, Bary);
			const varying FVector3f Difference = P1 - P;
			const varying float NormalDifference = VectorDot(Difference, Normal);
			if(NormalDifference >= Height || NormalDifference < MaxDepth)
			{
				continue;
			}

			const varying float TangentialDifference = VectorSize(Difference - NormalDifference * Normal);
			const uniform float TangentialFalloffMultiplier = 1.5f;
			const varying float TangentialFalloff = 1.f - TangentialDifference * OneOverTangentialFalloffDist;
			if(TangentialFalloff <= 0.f)
			{
				continue;
			}

			const float NormalDelta = Height - NormalDifference;
			const FVector3f RepulsionDelta = KinematicColliderStiffness * TangentialFalloff * NormalDelta *  Normal;

			P1 = P1 + RepulsionDelta;

			if (FrictionCoefficient > 0)
			{
				const FVector3f X1 = VectorGather(&ParticlesX[Index1]);
				const FVector3f V2 = VectorGather(&KinematicVelocities[Index2]);
				const FVector3f V3 = VectorGather(&KinematicVelocities[Index3]);
				const FVector3f V4 = VectorGather(&KinematicVelocities[Index4]);
				const FVector3f V = Bary.V[0] * V2 + Bary.V[1] * V3 + Bary.V[2] * V4;
				const FVector3f RelativeDisplacement = (P1 - X1) - V * Dt;
				const FVector3f RelativeDisplacementTangent = RelativeDisplacement - VectorDot(RelativeDisplacement,Normal) * Normal;
				const float RelativeDisplacementTangentLength = VectorSize(RelativeDisplacementTangent);
				const float PositionCorrection = min(NormalDelta * FrictionCoefficient, RelativeDisplacementTangentLength);
				const float CorrectionRatio = RelativeDisplacementTangentLength < FLOAT_SMALL_NUMBER ? 0.f : PositionCorrection / RelativeDisplacementTangentLength;
				const FVector3f FrictionDelta = -CorrectionRatio * RelativeDisplacementTangent;

				P1 = P1 + FrictionDelta;
			}
		}

		VectorScatter(&PandInvM[Index1], SetVector4(P1, IM));
	}
}

export void ApplyKinematicTriangleCollisionsWithMapsPAndInvM(
			uniform FVector4f PandInvM[],
			const uniform FVector4f KinematicPositions[],
			const uniform int32 KinematicCollidingParticles[],
			const uniform FIntVector KinematicColliderElements[],
			const uniform FIntVector TriangleMeshElements[],
			const uniform FVector2f& ParticleThicknessOffsetRange,
			const uniform float ParticleThicknessMapValues[],
			const uniform float KinematicColliderThickness,
			const uniform float KinematicColliderFalloffMultiplier,
			const uniform float KinematicColliderMaxDepthMultiplier,
			const uniform float KinematicColliderStiffness,
			const uniform int32 NumConstraints)
{
	foreach(Index = 0...NumConstraints)
	{
		const int32 Index1 = KinematicCollidingParticles[Index];

		const FVector4f PandInvM1 = VectorGather(&PandInvM[Index1]);
		varying FVector3f P1;
		varying float IM;
		UnzipPandInvM(PandInvM1, P1, IM);

		const varying FIntVector Elements = VectorLoad(&KinematicColliderElements[extract(Index,0)]);
		
		#pragma ignore warning(perf)
		const varying float ThicknessMapValue = ParticleThicknessMapValues[Index1];
		const varying float ParticleThickness = ParticleThicknessOffsetRange.V[0] + ThicknessMapValue * ParticleThicknessOffsetRange.V[1];
		const float Height = ParticleThickness + KinematicColliderThickness;
		const float OneOverTangentialFalloffDist = 1.f / max(KinematicColliderFalloffMultiplier * Height, FLOAT_KINDA_SMALL_NUMBER);
		const float MaxDepth = -Height * KinematicColliderMaxDepthMultiplier;

		for(uniform int32 EIndex = 0; EIndex < 3; ++EIndex)
		{
			const varying int32 ElementIndex = Elements.V[EIndex];
			if(ElementIndex == -1)
			{
				break;
			}

			const FIntVector Element = VectorGather(&TriangleMeshElements[ElementIndex]);

			const varying int32 Index2 = Element.V[0];
			const varying int32 Index3 = Element.V[1];
			const varying int32 Index4 = Element.V[2];
		
			const varying FVector4f PandInvM2 = VectorGather(&KinematicPositions[Index2]);
			const varying FVector4f PandInvM3 = VectorGather(&KinematicPositions[Index3]);
			const varying FVector4f PandInvM4 = VectorGather(&KinematicPositions[Index4]);

			varying FVector3f P2, P3, P4;
			varying float IM2, IM3, IM4;
			UnzipPandInvM(PandInvM2, P2, IM2);
			UnzipPandInvM(PandInvM3, P3, IM3);
			UnzipPandInvM(PandInvM4, P4, IM4);

			const varying FVector3f Normal = GetTriangleNormalFlipped(P2, P3, P4);

			varying FVector3f Bary;
			const varying FVector3f P = FindClosestPointAndBaryOnTriangle(P2, P3, P4, P1, Bary);
			const varying FVector3f Difference = P1 - P;
			const varying float NormalDifference = VectorDot(Difference, Normal);
			if(NormalDifference >= Height || NormalDifference < MaxDepth)
			{
				continue;
			}

			const varying float TangentialDifference = VectorSize(Difference - NormalDifference * Normal);
			const uniform float TangentialFalloffMultiplier = 1.5f;
			const varying float TangentialFalloff = 1.f - TangentialDifference * OneOverTangentialFalloffDist;
			if(TangentialFalloff <= 0.f)
			{
				continue;
			}

			const float NormalDelta = Height - NormalDifference;
			const FVector3f RepulsionDelta = KinematicColliderStiffness * TangentialFalloff * NormalDelta *  Normal;

			P1 = P1 + RepulsionDelta;
		}

		VectorScatter(&PandInvM[Index1], SetVector4(P1, IM));
	}
}

export void ApplyKinematicTriangleCollisionsWithFrictionAndMapsPAndInvM(
			uniform FVector4f PandInvM[],
			const uniform FVector3f ParticlesX[],
			const uniform FVector4f KinematicPositions[],
			const uniform FVector3f KinematicVelocities[],
			const uniform int32 KinematicCollidingParticles[],
			const uniform FIntVector KinematicColliderElements[],
			const uniform FIntVector TriangleMeshElements[],
			const uniform FVector2f& ParticleThicknessOffsetRange,
			const uniform float ParticleThicknessMapValues[],
			const uniform float Dt,
			const uniform float KinematicColliderThickness,
			const uniform float KinematicColliderFalloffMultiplier,
			const uniform float KinematicColliderMaxDepthMultiplier,
			const uniform float KinematicColliderStiffness, 
			const uniform bool bKinematicColliderFrictionHasMap,
			const uniform FVector2f& KinematicColliderFrictionCoefficientOffsetRange, 
			const uniform float KinematicColliderFrictionCoefficientMapValues[],
			const uniform int32 NumConstraints)
{
	foreach(Index = 0...NumConstraints)
	{
		const int32 Index1 = KinematicCollidingParticles[Index];

		const FVector4f PandInvM1 = VectorGather(&PandInvM[Index1]);
		varying FVector3f P1;
		varying float IM;
		UnzipPandInvM(PandInvM1, P1, IM);

		const varying FIntVector Elements = VectorLoad(&KinematicColliderElements[extract(Index,0)]);
		
		#pragma ignore warning(perf)
		const varying float ThicknessMapValue = ParticleThicknessMapValues[Index1];
		const varying float ParticleThickness = ParticleThicknessOffsetRange.V[0] + ThicknessMapValue * ParticleThicknessOffsetRange.V[1];
		const float Height = ParticleThickness + KinematicColliderThickness;
		const float OneOverTangentialFalloffDist = 1.f / max(KinematicColliderFalloffMultiplier * Height, FLOAT_KINDA_SMALL_NUMBER);
		const float MaxDepth = -Height * KinematicColliderMaxDepthMultiplier;

		varying float FrictionCoefficient = KinematicColliderFrictionCoefficientOffsetRange.V[0];
		if(bKinematicColliderFrictionHasMap)
		{
			#pragma ignore warning(perf)
			const varying float FrictionMapValue = KinematicColliderFrictionCoefficientMapValues[Index1];
			FrictionCoefficient += FrictionMapValue * KinematicColliderFrictionCoefficientOffsetRange.V[1];
		}

		for(uniform int32 EIndex = 0; EIndex < 3; ++EIndex)
		{
			const varying int32 ElementIndex = Elements.V[EIndex];
			if(ElementIndex == -1)
			{
				break;
			}

			const FIntVector Element = VectorGather(&TriangleMeshElements[ElementIndex]);

			const varying int32 Index2 = Element.V[0];
			const varying int32 Index3 = Element.V[1];
			const varying int32 Index4 = Element.V[2];
		
			const varying FVector4f PandInvM2 = VectorGather(&KinematicPositions[Index2]);
			const varying FVector4f PandInvM3 = VectorGather(&KinematicPositions[Index3]);
			const varying FVector4f PandInvM4 = VectorGather(&KinematicPositions[Index4]);

			varying FVector3f P2, P3, P4;
			varying float IM2, IM3, IM4;
			UnzipPandInvM(PandInvM2, P2, IM2);
			UnzipPandInvM(PandInvM3, P3, IM3);
			UnzipPandInvM(PandInvM4, P4, IM4);

			const varying FVector3f Normal = GetTriangleNormalFlipped(P2, P3, P4);

			varying FVector3f Bary;
			const varying FVector3f P = FindClosestPointAndBaryOnTriangle(P2, P3, P4, P1, Bary);
			const varying FVector3f Difference = P1 - P;
			const varying float NormalDifference = VectorDot(Difference, Normal);
			if(NormalDifference >= Height || NormalDifference < MaxDepth)
			{
				continue;
			}

			const varying float TangentialDifference = VectorSize(Difference - NormalDifference * Normal);
			const uniform float TangentialFalloffMultiplier = 1.5f;
			const varying float TangentialFalloff = 1.f - TangentialDifference * OneOverTangentialFalloffDist;
			if(TangentialFalloff <= 0.f)
			{
				continue;
			}

			const float NormalDelta = Height - NormalDifference;
			const FVector3f RepulsionDelta = KinematicColliderStiffness * TangentialFalloff * NormalDelta *  Normal;

			P1 = P1 + RepulsionDelta;


			if (FrictionCoefficient > 0)
			{
				const FVector3f X1 = VectorGather(&ParticlesX[Index1]);
				const FVector3f V2 = VectorGather(&KinematicVelocities[Index2]);
				const FVector3f V3 = VectorGather(&KinematicVelocities[Index3]);
				const FVector3f V4 = VectorGather(&KinematicVelocities[Index4]);
				const FVector3f V = Bary.V[0] * V2 + Bary.V[1] * V3 + Bary.V[2] * V4;
				const FVector3f RelativeDisplacement = (P1 - X1) - V * Dt;
				const FVector3f RelativeDisplacementTangent = RelativeDisplacement - VectorDot(RelativeDisplacement,Normal) * Normal;
				const float RelativeDisplacementTangentLength = VectorSize(RelativeDisplacementTangent);
				const float PositionCorrection = min(NormalDelta * FrictionCoefficient, RelativeDisplacementTangentLength);
				const float CorrectionRatio = RelativeDisplacementTangentLength < FLOAT_SMALL_NUMBER ? 0.f : PositionCorrection / RelativeDisplacementTangentLength;
				const FVector3f FrictionDelta = -CorrectionRatio * RelativeDisplacementTangent;

				P1 = P1 + FrictionDelta;
			}
		}

		VectorScatter(&PandInvM[Index1], SetVector4(P1, IM));
	}
}
