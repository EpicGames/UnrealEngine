// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/PBDLongRangeConstraints.h"
#include "ChaosLog.h"
#if INTEL_ISPC
#include "PBDLongRangeConstraints.ispc.generated.h"
#endif

#if INTEL_ISPC
static_assert(sizeof(ispc::FVector4f) == sizeof(Chaos::Softs::FPAndInvM), "sizeof(ispc::FVector4f) != sizeof(Chaos::Softs::FPAndInvM)");
static_assert(sizeof(ispc::FTether) == sizeof(Chaos::Softs::FPBDLongRangeConstraintsBase::FTether), "sizeof(ispc::FTether) != sizeof(Chaos::Softs::FPBDLongRangeConstraintsBase::FTether)");

// Support run-time toggling on supported platforms in non-shipping configurations
#if !UE_BUILD_SHIPPING || USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING
bool bChaos_LongRange_ISPC_Enabled = CHAOS_LONG_RANGE_ISPC_ENABLED_DEFAULT;
FAutoConsoleVariableRef CVarChaosLongRangeISPCEnabled(TEXT("p.Chaos.LongRange.ISPC"), bChaos_LongRange_ISPC_Enabled, TEXT("Whether to use ISPC optimizations in long range constraints"));
#endif
#endif

namespace Chaos::Softs {

void FPBDLongRangeConstraints::SetProperties(
	const FCollectionPropertyConstFacade& PropertyCollection,
	const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
	FSolverReal MeshScale)
{
	if (IsTetherStiffnessMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedValue(GetWeightedFloatTetherStiffness(PropertyCollection));
		if (IsTetherStiffnessStringDirty(PropertyCollection))
		{
			const FString& WeightMapName = GetTetherStiffnessString(PropertyCollection);
			Stiffness = FPBDStiffness(WeightedValue, WeightMaps.FindRef(WeightMapName), ParticleCount);
		}
		else
		{
			Stiffness.SetWeightedValue(WeightedValue);
		}
	}
	if (IsTetherScaleMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedValue = FSolverVec2(GetWeightedFloatTetherScale(PropertyCollection)).ClampAxes(MinTetherScale, MaxTetherScale) * MeshScale;
		if (IsTetherScaleStringDirty(PropertyCollection))
		{
			const FString& WeightMapName = GetTetherScaleString(PropertyCollection);
			TetherScale = FPBDWeightMap(WeightedValue, WeightMaps.FindRef(WeightMapName), ParticleCount);
		}
		else
		{
			TetherScale.SetWeightedValue(WeightedValue);
		}
	}
}

template<typename SolverParticlesOrRange>
void FPBDLongRangeConstraints::Apply(SolverParticlesOrRange& Particles, const FSolverReal /*Dt*/) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPBDLongRangeConstraints_Apply);
	SCOPE_CYCLE_COUNTER(STAT_PBD_LongRange);
	const int32 MinParallelSize = GetMinParallelBatchSize();

	if (!Stiffness.HasWeightMap())
	{
		const FSolverReal ExpStiffnessValue = (FSolverReal)Stiffness;
		if (!TetherScale.HasWeightMap())
		{
			const FSolverReal ScaleValue = (FSolverReal)TetherScale;
#if INTEL_ISPC
			if (bRealTypeCompatibleWithISPC && bChaos_LongRange_ISPC_Enabled)
			{
				// Run particles in parallel, and batch in sequence to avoid a race condition when updating the same particle from different tethers
				for (const TConstArrayView<FTether>& TetherBatch : Tethers)
				{
					ispc::ApplyLongRangeConstraints(
						(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
						(const ispc::FTether*)TetherBatch.GetData(),
						ExpStiffnessValue,
						ScaleValue,
						TetherBatch.Num(),
						ParticleOffset);
				}
			}
			else
#endif
			{
				// Run particles in parallel, and batch in sequence to avoid a race condition when updating the same particle from different tethers
				for (const TConstArrayView<FTether>& TetherBatch : Tethers)
				{
					PhysicsParallelFor(TetherBatch.Num(), [this, &Particles, &TetherBatch, ExpStiffnessValue, ScaleValue](int32 Index)
						{
							const FTether& Tether = TetherBatch[Index];
							Particles.P(GetEndParticle(Tether)) += ExpStiffnessValue * GetDelta(Particles, Tether, ScaleValue);
						}, TetherBatch.Num() < MinParallelSize);
				}
			}
		}
		else  // HasScaleWeightMap
		{
#if INTEL_ISPC
			if (bRealTypeCompatibleWithISPC && bChaos_LongRange_ISPC_Enabled)
			{
				// Run particles in parallel, and batch in sequence to avoid a race condition when updating the same particle from different tethers
				for (const TConstArrayView<FTether>& TetherBatch : Tethers)
				{
					ispc::ApplyLongRangeConstraintsScaleWeightmap(
						(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
						(const ispc::FTether*)TetherBatch.GetData(),
						ExpStiffnessValue,
						TetherScale.GetIndices().GetData(),
						TetherScale.GetTable().GetData(),
						TetherBatch.Num(),
						ParticleOffset);
				}
			}
			else
#endif
			{
				// Run particles in parallel, and batch in sequence to avoid a race condition when updating the same particle from different tethers
				for (const TConstArrayView<FTether>& TetherBatch : Tethers)
				{
					PhysicsParallelFor(TetherBatch.Num(), [this, &Particles, &TetherBatch, ExpStiffnessValue](int32 Index)
						{
							const FTether& Tether = TetherBatch[Index];
							const int32 LocalParticleIndex = GetEndIndex(Tether);
							const FSolverReal ScaleValue = TetherScale[LocalParticleIndex];
							Particles.P(ParticleOffset + LocalParticleIndex) += ExpStiffnessValue * GetDelta(Particles, Tether, ScaleValue);
						}, TetherBatch.Num() < MinParallelSize);
				}
			}
		}
	}
	else  // HasStiffnessWeighmap
	{
		if (!TetherScale.HasWeightMap())
		{
			const FSolverReal ScaleValue = (FSolverReal)TetherScale;

#if INTEL_ISPC
			if (bRealTypeCompatibleWithISPC && bChaos_LongRange_ISPC_Enabled)
			{
				// Run particles in parallel, and batch in sequence to avoid a race condition when updating the same particle from different tethers
				for (const TConstArrayView<FTether>& TetherBatch : Tethers)
				{
					ispc::ApplyLongRangeConstraintsStiffnessWeightmap(
						(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
						(const ispc::FTether*)TetherBatch.GetData(),
						Stiffness.GetIndices().GetData(),
						Stiffness.GetTable().GetData(),
						ScaleValue,
						TetherBatch.Num(),
						ParticleOffset);
				}
			}
			else
#endif
			{
				// Run particles in parallel, and batch in sequence to avoid a race condition when updating the same particle from different tethers
				for (const TConstArrayView<FTether>& TetherBatch : Tethers)
				{
					PhysicsParallelFor(TetherBatch.Num(), [this, &Particles, &TetherBatch, ScaleValue](int32 Index)
						{
							const FTether& Tether = TetherBatch[Index];
							const int32 LocalParticleIndex = GetEndIndex(Tether);
							const FSolverReal ExpStiffnessValue = Stiffness[LocalParticleIndex];
							Particles.P(ParticleOffset + LocalParticleIndex) += ExpStiffnessValue * GetDelta(Particles, Tether, ScaleValue);
						}, TetherBatch.Num() < MinParallelSize);
				}
			}
		}
		else // HasScaleWeightMap
		{
#if INTEL_ISPC
			if (bRealTypeCompatibleWithISPC && bChaos_LongRange_ISPC_Enabled)
			{
				// Run particles in parallel, and batch in sequence to avoid a race condition when updating the same particle from different tethers
				for (const TConstArrayView<FTether>& TetherBatch : Tethers)
				{
					ispc::ApplyLongRangeConstraintsStiffnessScaleWeightmaps(
						(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
						(const ispc::FTether*)TetherBatch.GetData(),
						Stiffness.GetIndices().GetData(),
						Stiffness.GetTable().GetData(),
						TetherScale.GetIndices().GetData(),
						TetherScale.GetTable().GetData(),
						TetherBatch.Num(),
						ParticleOffset);
				}
			}
			else
#endif
			{
				// Run particles in parallel, and batch in sequence to avoid a race condition when updating the same particle from different tethers
				for (const TConstArrayView<FTether>& TetherBatch : Tethers)
				{
					PhysicsParallelFor(TetherBatch.Num(), [this, &Particles, &TetherBatch](int32 Index)
						{
							const FTether& Tether = TetherBatch[Index];
							const int32 LocalParticleIndex = GetEndIndex(Tether);
							const FSolverReal ExpStiffnessValue = Stiffness[LocalParticleIndex];
							const FSolverReal ScaleValue = TetherScale[LocalParticleIndex];
							Particles.P(ParticleOffset + LocalParticleIndex) += ExpStiffnessValue * GetDelta(Particles, Tether, ScaleValue);
						}, TetherBatch.Num() < MinParallelSize);
				}
			}
		}
	}
}
template CHAOS_API void FPBDLongRangeConstraints::Apply(FSolverParticles& Particles, const FSolverReal /*Dt*/) const;
template CHAOS_API void FPBDLongRangeConstraints::Apply(FSolverParticlesRange& Particles, const FSolverReal /*Dt*/) const;

}  // End namespace Chaos::Softs
