// Copyright Epic Games, Inc. All Rights Reserved.

#define EXPLICIT_VECTOR4 1

#include "Math/Vector.isph"
#include "Chaos/PBDSofts.isph"

struct FTether
{
	int32 Start;
	int32 End;
	float RefLength;
};

static inline varying FTether LoadTetherHelper(const uniform FTether *uniform SrcPtr)
{
	varying FTether Tether;
	const varying FIntVector Loader = VectorLoad((const uniform FIntVector *uniform)SrcPtr);
	Tether.Start = Loader.V[0];
	Tether.End = Loader.V[1];
	Tether.RefLength = floatbits(Loader.V[2]);
	return Tether;
}

export void ApplyLongRangeConstraints(uniform FVector4f PandInvM[],
										const uniform FTether Tethers[],
										const uniform float Stiffness,
										const uniform float Scale,
										const uniform int32 NumConstraints,
										const uniform int32 ParticleOffset)
{
	varying FTether Tether;

#if HW_GATHER_SUPPORTED == 1
	if(programIndex < NumConstraints)
	{
		Tether = LoadTetherHelper(&Tethers[0]);
	}
#endif

	foreach(i = 0 ... NumConstraints)
	{
#if HW_GATHER_SUPPORTED == 0
		Tether = LoadTetherHelper(&Tethers[extract(i, 0)]);
#endif
		const varying int32 Start = Tether.Start;
		const varying int32 End = Tether.End;
		const varying float RefLength = Tether.RefLength;

		const varying int32 i1 = Start + ParticleOffset;
		const varying int32 i2 = End + ParticleOffset;

		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[i1]);
		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[i2]);

		varying FVector3f P1, P2;
		varying float InvM1, InvM2;
		UnzipPandInvM(PandInvM1, P1, InvM1);
		UnzipPandInvM(PandInvM2, P2, InvM2);

#if HW_GATHER_SUPPORTED == 1
		if(i + programCount < NumConstraints)
		{
			Tether = LoadTetherHelper(&Tethers[extract(i, 0) + programCount]);
		}
#endif

		FVector3f Direction = P1 - P2;
		const float Length = VectorSize(Direction);
		Direction = VectorSelect(Length < FLOAT_KINDA_SMALL_NUMBER, FloatForwardVector, Direction / Length);

		const float Offset = Length - RefLength * Scale;
		const FVector3f ComputedDelta = Stiffness * Offset * Direction;

		const FVector3f Delta = VectorSelect(Offset < FLOAT_ZERO, FloatZeroVector, ComputedDelta);

		VectorScatter(&PandInvM[i2], SetVector4(P2 + Delta, InvM2));
	}
}

export void ApplyLongRangeConstraintsStiffnessWeightmap(uniform FVector4f PandInvM[],
														const uniform FTether Tethers[],
														const uniform uint8 StiffnessIndices[],
														const uniform float StiffnessTable[],
														const uniform float Scale,
														const uniform int32 NumConstraints,
														const uniform int32 ParticleOffset)
{
	varying FTether Tether;

#if HW_GATHER_SUPPORTED == 1
	if(programIndex < NumConstraints)
	{
		Tether = LoadTetherHelper(&Tethers[0]);
	}
#endif

	foreach(i = 0 ... NumConstraints)
	{
#if HW_GATHER_SUPPORTED == 0
		Tether = LoadTetherHelper(&Tethers[extract(i, 0)]);
#endif
		const varying int32 Start = Tether.Start;
		const varying int32 End = Tether.End;
		const varying float RefLength = Tether.RefLength;

		#pragma ignore warning(perf)
		const varying uint8 StiffnessIndex = StiffnessIndices[End];

		const varying int32 i1 = Start + ParticleOffset;
		const varying int32 i2 = End + ParticleOffset;

		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[i1]);
		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[i2]);

		varying FVector3f P1, P2;
		varying float InvM1, InvM2;
		UnzipPandInvM(PandInvM1, P1, InvM1);
		UnzipPandInvM(PandInvM2, P2, InvM2);

#if HW_GATHER_SUPPORTED == 1
		if(i + programCount < NumConstraints)
		{
			Tether = LoadTetherHelper(&Tethers[extract(i, 0) + programCount]);
		}
#endif

		#pragma ignore warning(perf)
		const varying float Stiffness = StiffnessTable[StiffnessIndex];

		FVector3f Direction = P1 - P2;
		const float Length = VectorSize(Direction);
		Direction = VectorSelect(Length < FLOAT_KINDA_SMALL_NUMBER, FloatForwardVector, Direction / Length);

		const float Offset = Length - RefLength * Scale;
		const FVector3f ComputedDelta = Stiffness * Offset * Direction;

		const FVector3f Delta = VectorSelect(Offset < FLOAT_ZERO, FloatZeroVector, ComputedDelta);

		VectorScatter(&PandInvM[i2], SetVector4(P2 + Delta, InvM2));
	}
}

export void ApplyLongRangeConstraintsScaleWeightmap(uniform FVector4f PandInvM[],
													const uniform FTether Tethers[],
													const uniform float Stiffness,
													const uniform uint8 ScaleIndices[],
													const uniform float ScaleTable[],
													const uniform int32 NumConstraints,
													const uniform int32 ParticleOffset)
{
	varying FTether Tether;

#if HW_GATHER_SUPPORTED == 1
	if(programIndex < NumConstraints)
	{
		Tether = LoadTetherHelper(&Tethers[0]);
	}
#endif

	foreach(i = 0 ... NumConstraints)
	{
#if HW_GATHER_SUPPORTED == 0
		Tether = LoadTetherHelper(&Tethers[extract(i, 0)]);
#endif
		const varying int32 Start = Tether.Start;
		const varying int32 End = Tether.End;
		const varying float RefLength = Tether.RefLength;

		#pragma ignore warning(perf)
		const varying uint8 ScaleIndex = ScaleIndices[Tether.End];

		const varying int32 i1 = Start + ParticleOffset;
		const varying int32 i2 = End + ParticleOffset;

		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[i1]);
		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[i2]);

		varying FVector3f P1, P2;
		varying float InvM1, InvM2;
		UnzipPandInvM(PandInvM1, P1, InvM1);
		UnzipPandInvM(PandInvM2, P2, InvM2);

#if HW_GATHER_SUPPORTED == 1
		if(i + programCount < NumConstraints)
		{
			Tether = LoadTetherHelper(&Tethers[extract(i, 0) + programCount]);
		}
#endif

		#pragma ignore warning(perf)
		const varying float Scale = ScaleTable[ScaleIndex];

		FVector3f Direction = P1 - P2;
		const float Length = VectorSize(Direction);
		Direction = VectorSelect(Length < FLOAT_KINDA_SMALL_NUMBER, FloatForwardVector, Direction / Length);

		const float Offset = Length - RefLength * Scale;
		const FVector3f ComputedDelta = Stiffness * Offset * Direction;

		const FVector3f Delta = VectorSelect(Offset < FLOAT_ZERO, FloatZeroVector, ComputedDelta);

		VectorScatter(&PandInvM[i2], SetVector4(P2 + Delta, InvM2));
	}
}


export void ApplyLongRangeConstraintsStiffnessScaleWeightmaps(uniform FVector4f PandInvM[],
																const uniform FTether Tethers[],
																const uniform uint8 StiffnessIndices[],
																const uniform float StiffnessTable[],
																const uniform uint8 ScaleIndices[],
																const uniform float ScaleTable[],
																const uniform int32 NumConstraints,
																const uniform int32 ParticleOffset)
{
	varying FTether Tether;

#if HW_GATHER_SUPPORTED == 1
	if(programIndex < NumConstraints)
	{
		Tether = LoadTetherHelper(&Tethers[0]);
	}
#endif

	foreach(i = 0 ... NumConstraints)
	{
#if HW_GATHER_SUPPORTED == 0
		Tether = LoadTetherHelper(&Tethers[extract(i, 0)]);
#endif
		const varying int32 Start = Tether.Start;
		const varying int32 End = Tether.End;
		const varying float RefLength = Tether.RefLength;

		#pragma ignore warning(perf)
		const varying uint8 StiffnessIndex = StiffnessIndices[Tether.End];
		#pragma ignore warning(perf)
		const varying uint8 ScaleIndex = ScaleIndices[Tether.End];

		const varying int32 i1 = Start + ParticleOffset;
		const varying int32 i2 = End + ParticleOffset;

		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[i1]);
		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[i2]);

		varying FVector3f P1, P2;
		varying float InvM1, InvM2;
		UnzipPandInvM(PandInvM1, P1, InvM1);
		UnzipPandInvM(PandInvM2, P2, InvM2);

#if HW_GATHER_SUPPORTED == 1
		if(i + programCount < NumConstraints)
		{
			Tether = LoadTetherHelper(&Tethers[extract(i, 0) + programCount]);
		}
#endif

		#pragma ignore warning(perf)
		const varying float Stiffness = StiffnessTable[StiffnessIndex];
		#pragma ignore warning(perf)
		const varying float Scale = ScaleTable[ScaleIndex];

		FVector3f Direction = P1 - P2;
		const float Length = VectorSize(Direction);
		Direction = VectorSelect(Length < FLOAT_KINDA_SMALL_NUMBER, FloatForwardVector, Direction / Length);

		const float Offset = Length - RefLength * Scale;
		const FVector3f ComputedDelta = Stiffness * Offset * Direction;

		const FVector3f Delta = VectorSelect(Offset < FLOAT_ZERO, FloatZeroVector, ComputedDelta);

		VectorScatter(&PandInvM[i2], SetVector4(P2 + Delta, InvM2));
	}
}
