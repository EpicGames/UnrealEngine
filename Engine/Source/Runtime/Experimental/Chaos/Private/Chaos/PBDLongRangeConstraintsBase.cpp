// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/PBDLongRangeConstraintsBase.h"

#include "Chaos/Map.h"
#include "Chaos/Vector.h"
#include "Chaos/Framework/Parallel.h"
#include "ChaosStats.h"
#include "HAL/IConsoleManager.h"

DECLARE_CYCLE_STAT(TEXT("Chaos Cloth Compute Geodesic Constraints"), STAT_ChaosClothComputeGeodesicConstraints, STATGROUP_Chaos);

int32 ChaosPBDLongRangeConstraintsMinParallelBatchSize = 500;

FAutoConsoleVariableRef CVarChaosPBDLongRangeConstraintsMinParallelBatchSize(
	TEXT("p.Chaos.PBDLongRangeConstraints.MinParallelBatchSize"),
	ChaosPBDLongRangeConstraintsMinParallelBatchSize,
	TEXT("The minimum number of long range tethers in a batch to process in parallel."),
	ECVF_Cheat);

namespace Chaos::Softs {

int32 FPBDLongRangeConstraintsBase::GetMinParallelBatchSize()
{
	return ChaosPBDLongRangeConstraintsMinParallelBatchSize;
}

FPBDLongRangeConstraintsBase::FPBDLongRangeConstraintsBase(
	const FSolverParticlesRange& Particles,
	const TArray<TConstArrayView<TTuple<int32, int32, FRealSingle>>>& InTethers,
	const TConstArrayView<FRealSingle>& StiffnessMultipliers,
	const TConstArrayView<FRealSingle>& ScaleMultipliers,
	const FSolverVec2& InStiffness,
	const FSolverVec2& InScale,
	FSolverReal MaxStiffness,
	FSolverReal MeshScale)
	: Tethers(InTethers)
	, ParticleOffset(0)
	, ParticleCount(Particles.GetRangeSize())
	, Stiffness(
		InStiffness,
		StiffnessMultipliers,
		ParticleCount,
		FPBDStiffness::DefaultTableSize,
		FPBDStiffness::DefaultParameterFitBase,
		MaxStiffness)
	, TetherScale(
		InScale.ClampAxes(MinTetherScale, MaxTetherScale)* MeshScale,
		ScaleMultipliers,
		ParticleCount)
{
	// Apply default properties
	ApplyProperties((FSolverReal)(1. / FPBDStiffness::ParameterFrequency), 1);
}

FPBDLongRangeConstraintsBase::FPBDLongRangeConstraintsBase(
	const FSolverParticles& Particles,
	const int32 InParticleOffset,
	const int32 InParticleCount,
	const TArray<TConstArrayView<TTuple<int32, int32, FRealSingle>>>& InTethers,
	const TConstArrayView<FRealSingle>& StiffnessMultipliers,
	const TConstArrayView<FRealSingle>& ScaleMultipliers,
	const FSolverVec2& InStiffness,
	const FSolverVec2& InScale,
	FSolverReal MaxStiffness,
	FSolverReal MeshScale)
	: Tethers(InTethers)
	, ParticleOffset(InParticleOffset)
	, ParticleCount(InParticleCount)
	, Stiffness(
		InStiffness,
		StiffnessMultipliers,
		InParticleCount,
		FPBDStiffness::DefaultTableSize,
		FPBDStiffness::DefaultParameterFitBase,
		MaxStiffness)
	, TetherScale(
		InScale.ClampAxes(MinTetherScale, MaxTetherScale) * MeshScale,
		ScaleMultipliers,
		InParticleCount)
{
	// Apply default properties
	ApplyProperties((FSolverReal)(1. / FPBDStiffness::ParameterFrequency), 1);
}

}  // End namespace Chaos::Softs

