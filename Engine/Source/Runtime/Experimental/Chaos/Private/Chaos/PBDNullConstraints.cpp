// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/PBDNullConstraints.h"
#include "Chaos/Island/IslandManager.h"
#include "Chaos/ParticleHandle.h"

namespace Chaos
{
	FPBDNullConstraints::FPBDNullConstraints()
		: TPBDIndexedConstraintContainer<FPBDNullConstraints>(FPBDNullConstraintHandle::StaticType())
	{
	}

	FPBDNullConstraintHandle* FPBDNullConstraints::AddConstraint(const TVec2<FGeometryParticleHandle*>& InConstraintedParticles)
	{
		const int32 ConstraintIndex = Constraints.Emplace(FPBDNullConstraint(InConstraintedParticles));
		const int32 HandleIndex = Handles.Emplace(HandleAllocator.AllocHandle(this, ConstraintIndex));
		check(ConstraintIndex == HandleIndex);

		if (InConstraintedParticles[0] != nullptr)
		{
			InConstraintedParticles[0]->AddConstraintHandle(Handles[HandleIndex]);
		}
		if (InConstraintedParticles[1] != nullptr)
		{
			InConstraintedParticles[1]->AddConstraintHandle(Handles[HandleIndex]);
		}

		return Handles[HandleIndex];
	}

	void FPBDNullConstraints::AddConstraintsToGraph(Private::FPBDIslandManager& IslandManager)
	{
		IslandManager.AddContainerConstraints(*this);
	}
}