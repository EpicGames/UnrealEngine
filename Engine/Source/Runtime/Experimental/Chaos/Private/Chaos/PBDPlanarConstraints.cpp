// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/PBDPlanarConstraints.h"
#include "Chaos/PBDSoftsSolverParticles.h"
#include "Chaos/Framework/Parallel.h"
#include "ChaosStats.h"
#include "ChaosLog.h"
#include "HAL/IConsoleManager.h"

#if INTEL_ISPC
#include "PBDPlanarConstraints.ispc.generated.h"
#endif

#if INTEL_ISPC && !UE_BUILD_SHIPPING
static_assert(sizeof(ispc::FVector4f) == sizeof(Chaos::Softs::FPAndInvM), "sizeof(ispc::FVector4f) != sizeof(Chaos::Softs::FPAndInvM)");
static_assert(sizeof(ispc::FVector3f) == sizeof(Chaos::Softs::FSolverVec3), "sizeof(ispc::FVector3f) != sizeof(Chaos::Softs::FSolverVec3)");

bool bChaos_Planar_ISPC_Enabled = CHAOS_PLANAR_ISPC_ENABLED_DEFAULT;
FAutoConsoleVariableRef CVarChaosPlanarISPCEnabled(TEXT("p.Chaos.Planar.ISPC"), bChaos_Planar_ISPC_Enabled, TEXT("Whether to use ISPC optimizations in Planar constraints"));
#endif

namespace Chaos::Softs {

// @todo(chaos): the parallel threshold (or decision to run parallel) should probably be owned by the solver and passed to the constraint container
static int32 Chaos_Planar_ParallelConstraintCount = 32;
#if !UE_BUILD_SHIPPING
FAutoConsoleVariableRef CVarChaosPlanarParallelConstraintCount(TEXT("p.Chaos.Planar.ParallelConstraintCount"), Chaos_Planar_ParallelConstraintCount, TEXT("If we have more constraints than this, use parallel-for in Apply."));
#endif

void FPBDPlanarConstraints::Apply(FSolverParticlesRange& Particles, const FSolverReal Dt) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPBDPlanarConstraint_Apply);

	check(TargetPositions.Num() == UniqueConstraintIndices.Num());
	check(TargetNormals.Num() == UniqueConstraintIndices.Num());
	check(TargetVelocities.IsEmpty() || TargetVelocities.Num() == UniqueConstraintIndices.Num());
	checkSlow(UniqueConstraintIndices.Num() == TSet<int32>(UniqueConstraintIndices).Num());

	const bool bWithFriction = FrictionCoefficient > (FSolverReal)UE_KINDA_SMALL_NUMBER && TargetVelocities.Num() == UniqueConstraintIndices.Num();
	if (UniqueConstraintIndices.Num() > Chaos_Planar_ParallelConstraintCount)
	{
#if INTEL_ISPC
		if ( bRealTypeCompatibleWithISPC && bChaos_Planar_ISPC_Enabled)
		{
			if (bWithFriction)
			{
				ispc::ApplyPBDPlanarConstraintsWithFriction(
					(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
					(const ispc::FVector3f*)Particles.XArray().GetData(),
					UniqueConstraintIndices.GetData(),
					(const ispc::FVector3f*)TargetPositions.GetData(),
					(const ispc::FVector3f*)TargetNormals.GetData(),
					(const ispc::FVector3f*)TargetVelocities.GetData(),
					FrictionCoefficient,
					Dt,
					UniqueConstraintIndices.Num()
				);
			}
			else
			{
				ispc::ApplyPBDPlanarConstraintsNoFriction(
					(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
					UniqueConstraintIndices.GetData(),
					(const ispc::FVector3f*)TargetPositions.GetData(),
					(const ispc::FVector3f*)TargetNormals.GetData(),
					UniqueConstraintIndices.Num()
				);
			}
		}
		else
#endif
		{
			if (bWithFriction)
			{
				PhysicsParallelFor(UniqueConstraintIndices.Num(), [this, &Particles, Dt](int32 ConstraintIndex)
					{
						ApplyHelperWithFriction(Particles, Dt, ConstraintIndex);
					});
			}
			else
			{
				PhysicsParallelFor(UniqueConstraintIndices.Num(), [this, &Particles, Dt](int32 ConstraintIndex)
					{
						ApplyHelperNoFriction(Particles, ConstraintIndex);
					});
			}
		}
	}
	else
	{
		if (bWithFriction)
		{
			for (int32 ConstraintIndex = 0; ConstraintIndex < UniqueConstraintIndices.Num(); ++ConstraintIndex)
			{
				ApplyHelperWithFriction(Particles, Dt, ConstraintIndex);
			}
		}
		else
		{
			for (int32 ConstraintIndex = 0; ConstraintIndex < UniqueConstraintIndices.Num(); ++ConstraintIndex)
			{
				ApplyHelperNoFriction(Particles, ConstraintIndex);
			}
		}
	}
}

void FPBDPlanarConstraints::ApplyHelperWithFriction(FSolverParticlesRange& Particles, const FSolverReal Dt, const int32 ConstraintIndex) const
{
	const int32 ParticleIndex = UniqueConstraintIndices[ConstraintIndex];
	if (Particles.InvM(ParticleIndex) > (FSolverReal)0.)
	{
		const FSolverVec3 Difference = TargetPositions[ConstraintIndex] - Particles.P(ParticleIndex);
		const FSolverReal Penetration = FSolverVec3::DotProduct(Difference, TargetNormals[ConstraintIndex]);
		if (Penetration > (FSolverReal)0.)
		{
			Particles.P(ParticleIndex) += Penetration * TargetNormals[ConstraintIndex];

			const FSolverVec3 RelativeDisplacement = Particles.P(ParticleIndex) - Particles.X(ParticleIndex) - TargetVelocities[ConstraintIndex] * Dt;
			const FSolverVec3 RelativeDisplacementTangent = RelativeDisplacement - FSolverVec3::DotProduct(RelativeDisplacement, TargetNormals[ConstraintIndex]) * TargetNormals[ConstraintIndex];
			const FSolverReal RelativeDisplacementTangentLength = RelativeDisplacementTangent.Size();
			if (RelativeDisplacementTangentLength >= UE_SMALL_NUMBER)
			{
				const FSolverReal PositionCorrection = FMath::Min(FrictionCoefficient * Penetration, RelativeDisplacementTangentLength);
				const FSolverReal CorrectionRatio = PositionCorrection / RelativeDisplacementTangentLength;
				Particles.P(ParticleIndex) -= CorrectionRatio * RelativeDisplacementTangent;
			}
		}
	}
}

void FPBDPlanarConstraints::ApplyHelperNoFriction(FSolverParticlesRange& Particles, const int32 ConstraintIndex) const
{
	const int32 ParticleIndex = UniqueConstraintIndices[ConstraintIndex];
	if (Particles.InvM(ParticleIndex) > (FSolverReal)0.)
	{
		const FSolverVec3 Difference = TargetPositions[ConstraintIndex] - Particles.P(ParticleIndex);
		const FSolverReal Penetration = FSolverVec3::DotProduct(Difference, TargetNormals[ConstraintIndex]);
		if (Penetration > (FSolverReal)0.)
		{
			Particles.P(ParticleIndex) += Penetration * TargetNormals[ConstraintIndex];
		}
	}
}
} // End namespace Chaos::Softs
