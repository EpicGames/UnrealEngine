// Copyright Epic Games, Inc. All Rights Reserved.

#define EXPLICIT_VECTOR4 1

#include "Math/Vector.isph"
#include "Chaos/PBDSofts.isph"

export void ApplyPBDPlanarConstraintsWithFriction(uniform FVector4f PandInvM[],
									const uniform FVector3f Xs[],
									const uniform int32 ConstraintIndices[],
									const uniform FVector3f TargetPositions[],
									const uniform FVector3f TargetNormals[],
									const uniform FVector3f TargetVelocities[],
									const uniform float FrictionCoefficient,
									const uniform float Dt,
									const uniform int32 NumConstraints)
{
	foreach(ConstraintIndex = 0 ... NumConstraints)
	{
		const varying int32 ParticleIndex = ConstraintIndices[ConstraintIndex];

		const varying FVector4f PAndInvM1 = VectorGather(&PandInvM[ParticleIndex]);

		varying FVector3f P;
		varying float InvM;
		UnzipPandInvM(PAndInvM1, P, InvM);

		if(all(InvM) == 0)
		{
			continue;
		}

		const varying FVector3f TargetPosition = VectorLoad(&TargetPositions[extract(ConstraintIndex,0)]);
		const varying FVector3f TargetNormal = VectorLoad(&TargetNormals[extract(ConstraintIndex,0)]);
		const varying FVector3f Difference = TargetPosition - P;
		const varying float Penetration = VectorDot(Difference, TargetNormal);

		if(InvM != 0 && Penetration > 0.f)
		{
			P = P + (Penetration * TargetNormal);

			const varying FVector3f TargetVelocity = VectorLoad(&TargetVelocities[extract(ConstraintIndex,0)]);
			const varying FVector3f X = VectorGather(&Xs[ParticleIndex]);

			const varying FVector3f RelativeDisplacement = P - X - TargetVelocity * Dt;
			const FVector3f RelativeDisplacementTangent = RelativeDisplacement - VectorDot(RelativeDisplacement, TargetNormal) * TargetNormal; // Project displacement into the tangential plane
			const float RelativeDisplacementTangentLength = VectorSize(RelativeDisplacementTangent);
			const float PositionCorrection = min(FrictionCoefficient * Penetration, RelativeDisplacementTangentLength);
			const float CorrectionRatio = select(RelativeDisplacementTangentLength < FLOAT_SMALL_NUMBER, 0.0f, PositionCorrection / RelativeDisplacementTangentLength);
			P = P - (CorrectionRatio * RelativeDisplacementTangent);

			VectorScatter(&PandInvM[ParticleIndex], SetVector4(P, InvM));
		}		
	}
}

export void ApplyPBDPlanarConstraintsNoFriction(uniform FVector4f PandInvM[],
									const uniform int32 ConstraintIndices[],
									const uniform FVector3f TargetPositions[],
									const uniform FVector3f TargetNormals[],
									const uniform int32 NumConstraints)
{
	foreach(ConstraintIndex = 0 ... NumConstraints)
	{
		const varying int32 ParticleIndex = ConstraintIndices[ConstraintIndex];

		const varying FVector4f PAndInvM1 = VectorGather(&PandInvM[ParticleIndex]);

		varying FVector3f P;
		varying float InvM;
		UnzipPandInvM(PAndInvM1, P, InvM);

		if(all(InvM) == 0)
		{
			continue;
		}

		const varying FVector3f TargetPosition = VectorLoad(&TargetPositions[extract(ConstraintIndex,0)]);
		const varying FVector3f TargetNormal = VectorLoad(&TargetNormals[extract(ConstraintIndex,0)]);
		const varying FVector3f Difference = TargetPosition - P;
		const varying float Penetration = VectorDot(Difference, TargetNormal);

		if(InvM != 0 && Penetration > 0.f)
		{
			P = P + (Penetration * TargetNormal);

			VectorScatter(&PandInvM[ParticleIndex], SetVector4(P, InvM));
		}
	}
}
