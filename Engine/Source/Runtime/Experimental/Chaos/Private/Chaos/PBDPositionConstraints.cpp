// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/PBDPositionConstraints.h"
#include "Chaos/Island/IslandManager.h"

namespace Chaos
{
	FParticlePair FPBDPositionConstraintHandle::GetConstrainedParticles() const
	{ 
		return ConcreteContainer()->GetConstrainedParticles(ConstraintIndex);
	}

	void FPBDPositionConstraints::AddConstraintsToGraph(Private::FPBDIslandManager& IslandManager)
	{
		IslandManager.AddContainerConstraints(*this);
	}

	void FPBDPositionConstraints::AddBodies(FSolverBodyContainer& SolverBodyContainer)
	{
		for (int32 ConstraintIndex = 0; ConstraintIndex < NumConstraints(); ++ConstraintIndex)
		{
			AddBodies(ConstraintIndex, SolverBodyContainer);
		}
	}

	void FPBDPositionConstraints::ScatterOutput(const FReal Dt)
	{
		for (int32 ConstraintIndex = 0; ConstraintIndex < NumConstraints(); ++ConstraintIndex)
		{
			ConstraintSolverBodies[ConstraintIndex] = nullptr;
		}
	}

	void FPBDPositionConstraints::ApplyPositionConstraints(const FReal Dt, const int32 It, const int32 NumIts)
	{
		for (int32 ConstraintIndex = 0; ConstraintIndex < NumConstraints(); ++ConstraintIndex)
		{
			ApplySingle(Dt, ConstraintIndex);
		}
	}

	void FPBDPositionConstraints::AddBodies(const TArrayView<int32>& ConstraintIndices, FSolverBodyContainer& SolverBodyContainer)
	{
		for (int32 ConstraintIndex : ConstraintIndices)
		{
			AddBodies(ConstraintIndex, SolverBodyContainer);
		}
	}

	void FPBDPositionConstraints::ScatterOutput(const TArrayView<int32>& ConstraintIndices, const FReal Dt)
	{
		for (int32 ConstraintIndex : ConstraintIndices)
		{
			ConstraintSolverBodies[ConstraintIndex] = nullptr;
		}
	}

	void FPBDPositionConstraints::ApplyPositionConstraints(const TArrayView<int32>& ConstraintIndices, const FReal Dt, const int32 It, const int32 NumIts)
	{
		for (int32 ConstraintIndex : ConstraintIndices)
		{
			ApplySingle(Dt, ConstraintIndex);
		}
	}

	void FPBDPositionConstraints::AddBodies(const int32 ConstraintIndex, FSolverBodyContainer& SolverBodyContainer)
	{
		ConstraintSolverBodies[ConstraintIndex] = SolverBodyContainer.FindOrAdd(ConstrainedParticles[ConstraintIndex]);
	}
}
