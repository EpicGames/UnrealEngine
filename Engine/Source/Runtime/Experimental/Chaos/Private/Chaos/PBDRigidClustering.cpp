// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/PBDRigidClustering.h"

#include "Chaos/ErrorReporter.h"
#include "Chaos/ImplicitObjectTransformed.h"
#include "Chaos/ImplicitObjectUnion.h"
#include "Chaos/Levelset.h"
#include "Chaos/MassProperties.h"
#include "Chaos/PBDRigidsEvolution.h"
#include "Chaos/PBDCollisionConstraints.h"
#include "Chaos/PBDRigidClusteringAlgo.h"
#include "Chaos/Sphere.h"
#include "Chaos/UniformGrid.h"
#include "ChaosStats.h"
#include "Containers/Queue.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "Voronoi/Voronoi.h"
#include "Chaos/PBDRigidsEvolutionGBF.h"
#include "Chaos/PerParticleInitForce.h"
#include "Chaos/PerParticleEulerStepVelocity.h"
#include "Chaos/PerParticleEtherDrag.h"
#include "Chaos/PerParticlePBDEulerStep.h"
#include "Chaos/StrainModification.h"
#include "Chaos/ConvexOptimizer.h"
#include "PhysicsProxy/GeometryCollectionPhysicsProxy.h"
#include "PhysicsProxy/ClusterUnionPhysicsProxy.h"
#include "CoreMinimal.h"

namespace GeometryCollectionPhysicsProxy::Private
{
	extern CHAOS_API bool BuildGeometryForChildrenOnPT(const FSimulationParameters& Params);
}

namespace Chaos
{
	//
	//  Connectivity PVar
	//
	FRealSingle ClusterDistanceThreshold = 100.f;
	FAutoConsoleVariableRef CVarClusterDistance(TEXT("p.ClusterDistanceThreshold"), ClusterDistanceThreshold, TEXT("How close a cluster child must be to a contact to break off"));

	int32 UseConnectivity = 1;
	FAutoConsoleVariableRef CVarUseConnectivity(TEXT("p.UseConnectivity"), UseConnectivity, TEXT("Whether to use connectivity graph when breaking up clusters"));

	bool bCheckForInterclusterEdgesOnRelease = true;
	FAutoConsoleVariableRef CVarCheckForInterclusterEdgesOnRelease(TEXT("p.Chaos.CheckForInterclusterEdgesOnRelease"), bCheckForInterclusterEdgesOnRelease, TEXT("Whether to check for intercluster edges when removing a child from its parent cluster so that we can add the particle back into a cluster union."));

	bool bOnlyUseInterclusterEdgesAttachedToMainParticles = true;
	FAutoConsoleVariableRef CVarOnlyUseInterclusterEdgesAttachedToMainParticles(TEXT("p.Chaos.OnlyUseInterclusterEdgesAttachedToMainParticles"), bOnlyUseInterclusterEdgesAttachedToMainParticles, TEXT("If true, an intercluster edge must be directly attached to a main particle for the particle to remain a part of the cluster union."));

	int32 ComputeClusterCollisionStrains = 1;
	FAutoConsoleVariableRef CVarComputeClusterCollisionStrains(TEXT("p.ComputeClusterCollisionStrains"), ComputeClusterCollisionStrains, TEXT("Whether to use collision constraints when processing clustering."));

	int32 DeactivateClusterChildren = 0;
	FAutoConsoleVariableRef CVarDeactivateClusterChildren(TEXT("p.DeactivateClusterChildren"), DeactivateClusterChildren, TEXT("If children should be decativated when broken and put into another cluster."));

	int32 UseBoundingBoxForConnectionGraphFiltering = 0;
	FAutoConsoleVariableRef CVarUseBoundingBoxForConnectionGraphFiltering(TEXT("p.UseBoundingBoxForConnectionGraphFiltering"), UseBoundingBoxForConnectionGraphFiltering, TEXT("when on, use bounding box overlaps to filter connection during the connection graph generation [def: 0]"));

	float BoundingBoxMarginForConnectionGraphFiltering = 0;
	FAutoConsoleVariableRef CVarBoundingBoxMarginForConnectionGraphFiltering(TEXT("p.BoundingBoxMarginForConnectionGraphFiltering"), BoundingBoxMarginForConnectionGraphFiltering, TEXT("when UseBoundingBoxForConnectionGraphFiltering is on, the margin to use for the oevrlap test [def: 0]"));

	int32 GraphPropagationBasedCollisionImpulseProcessing = 0;
	FAutoConsoleVariableRef CVarGraphPropagationBasedCollisionImpulseProcessing(TEXT("p.GraphPropagationBasedCollisionImpulseProcessing"), GraphPropagationBasedCollisionImpulseProcessing, TEXT("when processing collision impulse toc ompute strain, pick the closest child from the impact point and propagate using the connection graph [def: 0]"));

	float GraphPropagationBasedCollisionFactor = 1;
	FAutoConsoleVariableRef CVarGraphPropagationBasedCollisionFactor(TEXT("p.GraphPropagationBasedCollisionFactor"), GraphPropagationBasedCollisionFactor, TEXT("when p.GraphPropagationBasedCollisionImpulseProcessing is on, the percentage [0-1] of remaining damage that is distributed to the connected pieces"));

	float RestoreBreakingMomentumPercent = .5;
	FAutoConsoleVariableRef CVarRestoreBreakingMomentumPercent(TEXT("p.RestoreBreakingMomentumPercent"), RestoreBreakingMomentumPercent, TEXT("When a rigid cluster is broken, objects that its in contact with will receive an impulse to restore this percent of their momentum prior to the break."));

	int32 ClusteringParticleReleaseThrottlingMinCount = INDEX_NONE;
	FAutoConsoleVariableRef CVarClusteringParticleReleaseThrottlingMinCount(TEXT("p.Clustering.ParticleReleaseThrottlingMinCount"), ClusteringParticleReleaseThrottlingMinCount, TEXT("Minimum number of active geometry collection to reach before clustering start to disable a percentage of the released particle per cluster"));

	int32 ClusteringParticleReleaseThrottlingMaxCount = INDEX_NONE;
	FAutoConsoleVariableRef CVarClusteringParticleReleaseThrottlingMaxCount(TEXT("p.Clustering.ParticleReleaseThrottlingMaxCount"), ClusteringParticleReleaseThrottlingMaxCount, TEXT("Maximum number of active geometry collection to reach before all released clustering disable all released particle instantly"));

	namespace CVars
	{
		extern CHAOS_API bool bChaosConvexSimplifyUnion;
		extern bool bChaosEnableOverrideSingleParticleTypeToCluster;
	}

	template <typename TProxy=FGeometryCollectionPhysicsProxy>
	TProxy* GetConcreteProxy(FPBDRigidClusteredParticleHandle* ClusteredParticle)
	{
		if (ClusteredParticle)
		{
			if (IPhysicsProxyBase* Proxy = ClusteredParticle->PhysicsProxy())
			{
				if (Proxy->GetType() == TProxy::ConcreteType())
				{
					return static_cast<TProxy*>(Proxy);
				}
			}
		}
		return nullptr;
	}

	template <typename TProxy=FGeometryCollectionPhysicsProxy>
	const TProxy* GetConcreteProxy(const FPBDRigidClusteredParticleHandle* ClusteredParticle)
	{
		if (ClusteredParticle)
		{
			if (const IPhysicsProxyBase* Proxy = ClusteredParticle->PhysicsProxy())
			{
				if (Proxy->GetType() == TProxy::ConcreteType())
				{
					return static_cast<const TProxy*>(Proxy);
				}
			}
		}
		return nullptr;
	}

	namespace
	{
		FPBDRigidClusteredParticleHandle* GetActiveParentParticle(FPBDRigidParticleHandle* Particle)
		{
			if (!Particle)
			{
				return nullptr;
			}

			FPBDRigidClusteredParticleHandle* Current = Particle->CastToClustered();

			while (Current)
			{
				if (!Current->Disabled())
				{
					break;
				}

				Current = Current->Parent();
			}

			return Current;
		}

		// compute a ratio (between 0 and 1) of released particles to release
		float GetRatioOfReleasedParticlesToDisable(const FRigidClustering::FRigidEvolution& Evolution)
		{
			const FChaosSolverDestructionSettings& DestructionSettings = Evolution.GetRigidClustering().GetDestructionSettings();
			const FPBDRigidsSOAs& ParticleStructures = Evolution.GetParticles();
			int32 NumActiveParticles = 0;
			NumActiveParticles += ParticleStructures.GetSleepingGeometryCollectionArray().Num();
			NumActiveParticles += ParticleStructures.GetDynamicGeometryCollectionArray().Num();

			const int32 Range = FMath::Max(0, (DestructionSettings.ClusteringParticleReleaseThrottlingMaxCount - DestructionSettings.ClusteringParticleReleaseThrottlingMinCount));
			const int32 OverMinCount = FMath::Max(0, (NumActiveParticles - DestructionSettings.ClusteringParticleReleaseThrottlingMinCount));

			if (Range > 0)
			{
				// clamp to 1, as OverMinCount can get larger than Range
				return FMath::Min(1.f, ((float)OverMinCount / (float)Range));
			}

			return 1.0f;
		}

		template<typename TParticleContainer>
		void GenericThrottleReleasedParticlesIfNecessary(TParticleContainer& Container, FRigidClustering& Clustering)
		{
			if (!Clustering.ShouldThrottleParticleRelease())
			{
				return;
			}

			const float RatioOfParticlesToDisable = GetRatioOfReleasedParticlesToDisable(Clustering.GetEvolution());
			const int32 NumberOfParticlesToDisable = (int32)((float)Container.Num() * RatioOfParticlesToDisable);
			if (NumberOfParticlesToDisable > 0)
			{ 
				int32 DisabledParticleCount = 0;
				for (auto ChildIt = Container.CreateIterator(); ChildIt; ++ChildIt)
				{
					if (FPBDRigidParticleHandle* Child = *ChildIt)
					{
						DisabledParticleCount++;
						Clustering.DisableParticle(Child);
						Clustering.GetEvolution().GetParticles().MarkTransientDirtyParticle(Child);
						ChildIt.RemoveCurrent();
					}
					if (DisabledParticleCount >= NumberOfParticlesToDisable)
					{
						break;
					}
				}
			}
		}
	}
	
	//==========================================================================
	// TPBDRigidClustering
	//==========================================================================

	FRigidClustering::FRigidClustering(FPBDRigidsEvolution& InEvolution, FPBDRigidClusteredParticles& InParticles, const TArray<ISimCallbackObject*>* InStrainModifiers)
		: MEvolution(InEvolution)
		, MParticles(InParticles)
		, ClusterUnionManager(*this, InEvolution)
		, MCollisionImpulseArrayDirty(true)
		, DoGenerateBreakingData(false)
		, MClusterConnectionFactor(1.0)
		, MClusterUnionConnectionType(FClusterCreationParameters::EConnectionMethod::DelaunayTriangulation)
		, StrainModifiers(InStrainModifiers)
	{}

	FRigidClustering::~FRigidClustering()
	{}

	DECLARE_CYCLE_STAT(TEXT("TPBDRigidClustering<>::CreateClusterParticle"), STAT_CreateClusterParticle, STATGROUP_Chaos);
	Chaos::FPBDRigidClusteredParticleHandle* FRigidClustering::CreateClusterParticle(
		const int32 ClusterGroupIndex, 
		TArray<Chaos::FPBDRigidParticleHandle*>&& Children, 
		const FClusterCreationParameters& Parameters, 
		const Chaos::FImplicitObjectPtr& ProxyGeometry, 
		const FRigidTransform3* ForceMassOrientation, 
		const FUniqueIdx* ExistingIndex)
	{
		SCOPE_CYCLE_COUNTER(STAT_CreateClusterParticle);

		Chaos::FPBDRigidClusteredParticleHandle* NewParticle = Parameters.ClusterParticleHandle;
		if (!NewParticle)
		{
			NewParticle = MEvolution.CreateClusteredParticles(1, ExistingIndex)[0]; // calls Evolution.DirtyParticle()
		}

		// Must do this so that the constraint graph knows about this particle 
		MEvolution.EnableParticle(NewParticle);
		NewParticle->SetCollisionGroup(INT_MAX);
		TopLevelClusterParents.Add(NewParticle);

		NewParticle->SetInternalCluster(false);
		NewParticle->SetClusterId(ClusterId(nullptr, Children.Num()));
		NewParticle->SetClusterGroupIndex(ClusterGroupIndex);
		NewParticle->SetInternalStrains(0.0);
		UpdateTopLevelParticle(NewParticle);
		NewParticle->SetIsAnchored(Parameters.bIsAnchored);

		// Update clustering data structures.
		if (MChildren.Contains(NewParticle))
		{
			MChildren[NewParticle] = MoveTemp(Children);
		}
		else
		{
			MChildren.Add(NewParticle, MoveTemp(Children));
		}

		const TArray<FPBDRigidParticleHandle*>& ChildrenArray = MChildren[NewParticle];
		TSet<FPBDRigidParticleHandle*> ChildrenSet(ChildrenArray);

		// Disable the children
		MEvolution.DisableParticles(reinterpret_cast<TSet<FGeometryParticleHandle*>&>(ChildrenSet));

		bool bClusterIsAsleep = true;
		bool bClusterIsOneWayInteraction = true;
		bool bClusterIsMACD = false;
		for (FPBDRigidParticleHandle* Child : ChildrenSet)
		{
			bClusterIsAsleep &= Child->Sleeping();
			bClusterIsOneWayInteraction &= Child->OneWayInteraction();
			bClusterIsMACD |= Child->MACDEnabled();

			if (FPBDRigidClusteredParticleHandle* ClusteredChild = Child->CastToClustered())
			{
				TopLevelClusterParents.Remove(ClusteredChild);
				TopLevelClusterParentsStrained.Remove(ClusteredChild);

				// Cluster group id 0 means "don't union with other things"
				// TODO: Use INDEX_NONE instead of 0?
				ClusteredChild->SetClusterGroupIndex(0);
				ClusteredChild->ClusterIds().Id = NewParticle;
				NewParticle->SetInternalStrains(NewParticle->GetInternalStrains() + ClusteredChild->GetInternalStrains());
				UpdateTopLevelParticle(NewParticle);

				NewParticle->SetCollisionImpulses(FMath::Max(NewParticle->CollisionImpulses(), ClusteredChild->CollisionImpulses()));

				const int32 NewCG = NewParticle->CollisionGroup();
				const int32 ChildCG = ClusteredChild->CollisionGroup();
				NewParticle->SetCollisionGroup(NewCG < ChildCG ? NewCG : ChildCG);
			}
		}
		if (ChildrenSet.Num())
		{
			NewParticle->SetInternalStrains(NewParticle->GetInternalStrains() / static_cast<FRealSingle>(ChildrenSet.Num()));
			UpdateTopLevelParticle(NewParticle);

			// NOTE: These property values are only known when we have children. They should be overwritten when 
			// children are added to an empty cluster, but we also shouldn't set non-default values before then
			NewParticle->SetSleeping(bClusterIsAsleep);
			NewParticle->SetOneWayInteraction(bClusterIsOneWayInteraction);
			NewParticle->SetMACDEnabled(bClusterIsMACD);
		}

		if (ForceMassOrientation)
		{
			NewParticle->SetX(ForceMassOrientation->GetTranslation());
			NewParticle->SetR(ForceMassOrientation->GetRotation());
		}
		
		UpdateClusterMassProperties(NewParticle, ChildrenSet);
		
		if (ForceMassOrientation == nullptr)
		{
			MoveClusterToMassOffset(NewParticle, EMassOffsetType::Position);
		}
		UpdateKinematicProperties(NewParticle, MChildren, MEvolution);
		UpdateGeometry(NewParticle, ChildrenSet, MChildren, ProxyGeometry, Parameters);

		GenerateConnectionGraph(NewParticle, Parameters);

		// Build the convex optimizer if required
		//FRigidClustering::BuildConvexOptimizer(NewParticle);

		auto AddToClusterUnion = [this](int32 ClusterID, FPBDRigidClusteredParticleHandle* Handle)
		{
			if (ClusterID <= 0)
			{
				return;
			}

			ClusterUnionManager.AddPendingExplicitIndexOperation(ClusterID, EClusterUnionOperation::AddReleased, { Handle });
		};

		if(ClusterGroupIndex)
		{
			AddToClusterUnion(ClusterGroupIndex, NewParticle);
		}

		return NewParticle;
	}

	DECLARE_CYCLE_STAT(TEXT("TPBDRigidClustering<>::AddParticlesToCluster"), STAT_AddParticlesToCluster, STATGROUP_Chaos);
	void
	FRigidClustering::AddParticlesToCluster(
		FPBDRigidClusteredParticleHandle* Cluster,
		const TArray<FPBDRigidParticleHandle*>& InChildren,
		const TMap<FPBDRigidParticleHandle*, FPBDRigidParticleHandle*>& ChildToParentMap)
	{
		SCOPE_CYCLE_COUNTER(STAT_AddParticlesToCluster);
		if (!Cluster || InChildren.IsEmpty())
		{
			return;
		}

		FRigidHandleArray& Children = MChildren.FindOrAdd(Cluster);
		
		// Note that we want to compute the internal strain on the cluster the same if we build it up incrementally as well as if we
		// build it all at the same time. The parent cluster's internal strain should be the average of all the child strains.
		// The easy way to compute the new average is the multiply the old average by the number of old elements, add in the new strains,
		// and then divide by the new total number of elements.
		Cluster->SetInternalStrains(Cluster->GetInternalStrains() * static_cast<FRealSingle>(Children.Num()));

		Children.Append(InChildren);

		// Disable all the input children since they no longer need to be simulated.
		TSet<FPBDRigidParticleHandle*> InChildrenSet(InChildren);
		for (FPBDRigidParticleHandle* Handle : InChildren)
		{
			if (FPBDRigidClusteredParticleHandle* ClusteredChild = Handle->CastToClustered())
			{
				if (FPBDRigidClusteredParticleHandle* ExistingParent = ClusteredChild->Parent())
				{
					if (ExistingParent != Cluster)
					{
						// This is needed in the case where we use intercluster edges with geometry collections that need to then get stuck into a cluster union.
						// It's possible due to replication ordering that we create an internal cluster surrounding the piece that we want to add to the cluster union first.
						RemoveChildFromParentAndChildrenArray(ClusteredChild, ExistingParent);
					}
				}

				TopLevelClusterParents.Remove(ClusteredChild);
				TopLevelClusterParentsStrained.Remove(ClusteredChild);

				ClusteredChild->ClusterIds().Id = Cluster;
				Cluster->SetInternalStrains(Cluster->GetInternalStrains() + ClusteredChild->GetInternalStrains());
				Cluster->SetCollisionGroup(FMath::Min(Cluster->CollisionGroup(), ClusteredChild->CollisionGroup()));
			}

			Cluster->AddPhysicsProxy(Handle->PhysicsProxy());
			if (Cluster->PhysicsProxy() == nullptr)
			{
				Cluster->SetPhysicsProxy(Handle->PhysicsProxy());
			}

			MEvolution.DisableParticle(Handle);
			MEvolution.GetParticles().MarkTransientDirtyParticle(Handle);
		}

		Cluster->ClusterIds().NumChildren = Children.Num();
		Cluster->SetInternalStrains(Cluster->GetInternalStrains() / static_cast<FRealSingle>(Children.Num()));
	}

	void FRigidClustering::BuildConvexOptimizer(FPBDRigidClusteredParticleHandle* Particle)
	{
		bool bHasOptimizer = false;
		if(Particle && Particle->GetGeometry() && CVars::bChaosConvexSimplifyUnion)
		{ 
			if(FImplicitObjectUnion* Union = Particle->GetGeometry()->template AsA<FImplicitObjectUnion>())
			{
				TBitArray<> bOptimizeConvexes;
				bOptimizeConvexes.Init(true, Particle->ShapesArray().Num());
				if (FGeometryCollectionPhysicsProxy* ConcreteProxy = GetConcreteProxy(Particle))
				{
					if(Particle->ShapesArray().Num() == 1)
					{ 
						bOptimizeConvexes[0] = ConcreteProxy->GetSimParameters().bOptimizeConvexes;
					}
				}
				else if (Chaos::FClusterUnion* ClusterUnion = ClusterUnionManager.FindClusterUnionFromParticle(Particle))
				{
					if(Particle->ShapesArray().Num() == ClusterUnion->ChildParticles.Num())
					{
						int32 ShapeIndex = 0;
						for (FPBDRigidParticleHandle* ChildHandle : ClusterUnion->ChildParticles)
						{
							if (FPBDRigidClusteredParticleHandle* ChildClustered = ChildHandle->CastToClustered())
							{
								if (FGeometryCollectionPhysicsProxy* ChildProxy = GetConcreteProxy(ChildClustered))
								{
									bOptimizeConvexes[ShapeIndex] = ChildProxy->GetSimParameters().bOptimizeConvexes;
								}
							}
							++ShapeIndex;
						}
					}
				}
				if(Union->GetNumLeafObjects() > 1)
				{ 
					if (!Particle->ConvexOptimizer())
					{
						Particle->ConvexOptimizer() = MakePimpl<Private::FConvexOptimizer>();
					}
					Particle->ConvexOptimizer()->SimplifyRootConvexes(Union,
						Particle->ShapesArray(), Particle->ObjectState(), bOptimizeConvexes);
					bHasOptimizer = Particle->ConvexOptimizer()->IsValid();
				}
			}
		}
		if(Particle && !bHasOptimizer)
		{	
			Particle->ConvexOptimizer().Reset();
		}
	}

	DECLARE_CYCLE_STAT(TEXT("TPBDRigidClustering<>::RemoveParticlesFromCluster"), STAT_RemoveParticlesFromCluster, STATGROUP_Chaos);
	void
	FRigidClustering::RemoveParticlesFromCluster(
		FPBDRigidClusteredParticleHandle* Cluster,
		const TArray<FPBDRigidParticleHandle*>& InChildren)
	{
		SCOPE_CYCLE_COUNTER(STAT_RemoveParticlesFromCluster);

		FRigidHandleArray& Children = MChildren.FindOrAdd(Cluster);
		TSet<IPhysicsProxyBase*> RemovedProxies;

		FRealSingle NewInternalStrain = Cluster->GetInternalStrains() * static_cast<FRealSingle>(Children.Num());

		for (FPBDRigidParticleHandle* Child : InChildren)
		{
			if (int32 Index = Children.Find(Child); Child && Index != INDEX_NONE)
			{
				RemovedProxies.Add(Child->PhysicsProxy());
				RemoveChildFromParent(Child, Cluster);

				if (FPBDRigidClusteredParticleHandle* ChildCluster = Child->CastToClustered())
				{
					NewInternalStrain -= ChildCluster->GetInternalStrains();
				}

				Children.RemoveAtSwap(Index);
				MEvolution.DirtyParticle(*Child);
				MEvolution.GetParticles().MarkTransientDirtyParticle(Child);
			}
		}

		Cluster->ClusterIds().NumChildren = Children.Num();

		// If we removed the last particle with a given physics proxy from a cluster, we need to remove that proxy from the proxy set.
		if (Children.IsEmpty())
		{
			Cluster->SetInternalStrains(FRealSingle(0.0));
			Cluster->ClearPhysicsProxies();
		}
		else
		{
			Cluster->SetInternalStrains(NewInternalStrain / static_cast<FRealSingle>(Children.Num()));

			// Unfortunately we still need to iterate through every child in the cluster to see if a particular physics proxy in the set is still valid.
			for (FPBDRigidParticleHandle* Child : Children)
			{
				RemovedProxies.Remove(Child->PhysicsProxy());
				if (RemovedProxies.IsEmpty())
				{
					break;
				}
			}

			for (IPhysicsProxyBase* Proxy : RemovedProxies)
			{
				Cluster->RemovePhysicsProxy(Proxy);
			}

			IPhysicsProxyBase* FallbackProxy = Children[0]->PhysicsProxy();
			if (RemovedProxies.Contains(Cluster->PhysicsProxy()))
			{
				Cluster->SetPhysicsProxy(FallbackProxy);
			}
		}

		MEvolution.DirtyParticle(*Cluster);
	}

	DECLARE_CYCLE_STAT(TEXT("TPBDRigidClustering<>::UpdateClusterParticlePropertiesFromChildren"), STAT_UpdateClusterParticlePropertiesFromChildren, STATGROUP_Chaos);
	void
	FRigidClustering::UpdateClusterParticlePropertiesFromChildren(
		FPBDRigidClusteredParticleHandle* Cluster,
		const FRigidHandleArray& Children,
		const TMap<FPBDRigidParticleHandle*, FPBDRigidParticleHandle*>& ChildToParentMap)
	{
		SCOPE_CYCLE_COUNTER(STAT_UpdateClusterParticlePropertiesFromChildren);
		// An initial pass through the children to transfer some of their cluster properties to their new parent.
		for (FPBDRigidParticleHandle* Child : Children)
		{
			if (FPBDRigidClusteredParticleHandle* ClusteredChild = Child->CastToClustered())
			{
				Cluster->SetInternalStrains(Cluster->GetInternalStrains() + ClusteredChild->GetInternalStrains());
				Cluster->SetCollisionImpulses(FMath::Max(Cluster->CollisionImpulses(), ClusteredChild->CollisionImpulses()));

				const int32 NewCG = Cluster->CollisionGroup();
				const int32 ChildCG = ClusteredChild->CollisionGroup();
				Cluster->SetCollisionGroup(NewCG < ChildCG ? NewCG : ChildCG);
			}

			FPBDRigidParticleHandle* const* OldParent = ChildToParentMap.Find(Child);
			FPBDRigidParticleHandle* ProxyParticle = (OldParent != nullptr) ? *OldParent : Child;
			MEvolution.DoInternalParticleInitilization(ProxyParticle, Cluster);
		}

		if (Cluster->ClusterIds().NumChildren > 0)
		{
			Cluster->SetInternalStrains(Cluster->GetInternalStrains() / static_cast<FRealSingle>(Cluster->ClusterIds().NumChildren));
		}
	}

	int32 UnionsHaveCollisionParticles = 0;
	FAutoConsoleVariableRef CVarUnionsHaveCollisionParticles(TEXT("p.UnionsHaveCollisionParticles"), UnionsHaveCollisionParticles, TEXT(""));

	bool
	FRigidClustering::ShouldUnionsHaveCollisionParticles()
	{
		return !!UnionsHaveCollisionParticles;
	}

	DECLARE_CYCLE_STAT(TEXT("TPBDRigidClustering<>::CreateClusterParticleFromClusterChildren"), STAT_CreateClusterParticleFromClusterChildren, STATGROUP_Chaos);
	Chaos::FPBDRigidClusteredParticleHandle* 
	FRigidClustering::CreateClusterParticleFromClusterChildren(
		TArray<FPBDRigidParticleHandle*>&& Children, 
		FPBDRigidClusteredParticleHandle* Parent, 
		const FRigidTransform3& ClusterWorldTM, 
		const FClusterCreationParameters& Parameters)
	{
		SCOPE_CYCLE_COUNTER(STAT_CreateClusterParticleFromClusterChildren);

		//This cluster is made up of children that are currently in a cluster. This means we don't need to update or disable as much
		Chaos::FPBDRigidClusteredParticleHandle* NewParticle = Parameters.ClusterParticleHandle;
		if (!NewParticle)
		{
			NewParticle = MEvolution.CreateClusteredParticles(1)[0]; // calls Evolution.DirtyParticle()
		}
		MEvolution.EnableParticle(NewParticle);

		NewParticle->SetCollisionGroup(INT_MAX);
		TopLevelClusterParents.Add(NewParticle);
		NewParticle->SetInternalCluster(true);
		NewParticle->SetClusterId(ClusterId(nullptr, Children.Num()));
		NewParticle->SetIsAnchored(false);
		if (Parent)
		{
			NewParticle->SetGravityEnabled(Parent->GravityEnabled());
			NewParticle->SetGravityGroupIndex(Parent->GravityGroupIndex());
			NewParticle->SetInertiaConditioningEnabled(Parent->InertiaConditioningEnabled());
			NewParticle->SetCCDEnabled(Parent->CCDEnabled());
			NewParticle->SetMACDEnabled(Parent->MACDEnabled());
			NewParticle->SetOneWayInteraction(Parent->OneWayInteraction());
			NewParticle->SetLinearEtherDrag(Parent->LinearEtherDrag());
			NewParticle->SetAngularEtherDrag(Parent->AngularEtherDrag());
			NewParticle->SetInitialOverlapDepenetrationVelocity(Parent->InitialOverlapDepenetrationVelocity());
			NewParticle->SetSleepThresholdMultiplier(Parent->SleepThresholdMultiplier());
			NewParticle->SetIterationSettings(Parent->IterationSettings());
		}

		for (auto& Constituent : Children)
		{
			MEvolution.DoInternalParticleInitilization(Constituent, NewParticle);
		}

		//
		// Update clustering data structures.
		//
		if (MChildren.Contains(NewParticle))
		{
			MChildren[NewParticle] = MoveTemp(Children);
		}
		else
		{
			MChildren.Add(NewParticle, MoveTemp(Children));
		}

		TArray<FPBDRigidParticleHandle*>& ChildrenArray = MChildren[NewParticle];
		//child transforms are out of date, need to update them. @todo(ocohen): if children transforms are relative we would not need to update this, but would simply have to do a final transform on the new cluster index
		// TODO(mlentine): Why is this not needed? (Why is it ok to have DeactivateClusterChildren==false?)
		if (DeactivateClusterChildren)
		{
			//TODO: avoid iteration just pass in a view
			TSet<FGeometryParticleHandle*> ChildrenHandles(static_cast<TArray<FGeometryParticleHandle*>>(ChildrenArray));
			MEvolution.DisableParticles(ChildrenHandles);
		}
		bool bClusterIsOneWayInteraction = true;
		bool bClusterIsMACD = false;

		for (FPBDRigidParticleHandle* Child : ChildrenArray)
		{
			if (FPBDRigidClusteredParticleHandle* ClusteredChild = Child->CastToClustered())
			{
				FRigidTransform3 ChildFrame = ClusteredChild->ChildToParent() * ClusterWorldTM;
				ClusteredChild->SetX(ChildFrame.GetTranslation());
				ClusteredChild->SetR(ChildFrame.GetRotation());
				ClusteredChild->ClusterIds().Id = NewParticle;
				ClusteredChild->SetClusterGroupIndex(0);
				if (DeactivateClusterChildren)
				{
					TopLevelClusterParents.Remove(ClusteredChild);
					TopLevelClusterParentsStrained.Remove(ClusteredChild);
				}

				NewParticle->SetCollisionImpulses(FMath::Max(NewParticle->CollisionImpulses(), ClusteredChild->CollisionImpulses()));

				const int32 NewCG = NewParticle->CollisionGroup();
				const int32 ChildCG = ClusteredChild->CollisionGroup();
				NewParticle->SetCollisionGroup(NewCG < ChildCG ? NewCG : ChildCG);

				bClusterIsOneWayInteraction &= Child->OneWayInteraction();
				bClusterIsMACD |= Child->MACDEnabled();
			}
		}

		FClusterCreationParameters NoCleanParams = Parameters;
		NoCleanParams.bCleanCollisionParticles = false;
		NoCleanParams.bCopyCollisionParticles = !!UnionsHaveCollisionParticles;

		TSet<FPBDRigidParticleHandle*> ChildrenSet(ChildrenArray);

		UpdateClusterMassProperties(NewParticle, ChildrenSet);
		MoveClusterToMassOffset(NewParticle, EMassOffsetType::Position);

		UpdateKinematicProperties(NewParticle, MChildren, MEvolution);

		UpdateGeometry(NewParticle, ChildrenSet, MChildren, FImplicitObjectPtr(nullptr), NoCleanParams);

		NewParticle->SetOneWayInteraction(bClusterIsOneWayInteraction);
		NewParticle->SetMACDEnabled(bClusterIsMACD);

		// Build the convex optimizer if required
		FRigidClustering::BuildConvexOptimizer(NewParticle);

		return NewParticle;
	}

	DECLARE_CYCLE_STAT(TEXT("TPBDRigidClustering<>::UnionClusterGroups"), STAT_UnionClusterGroups, STATGROUP_Chaos);
	void 
	FRigidClustering::UnionClusterGroups()
	{
		SCOPE_CYCLE_COUNTER(STAT_UnionClusterGroups);
		ClusterUnionManager.FlushPendingOperations();
	}

	DECLARE_CYCLE_STAT(TEXT("TPBDRigidClustering<>::DeactivateClusterParticle"), STAT_DeactivateClusterParticle, STATGROUP_Chaos);
	TSet<FPBDRigidParticleHandle*> 
	FRigidClustering::DeactivateClusterParticle(
		FPBDRigidClusteredParticleHandle* ClusteredParticle)
	{
		SCOPE_CYCLE_COUNTER(STAT_DeactivateClusterParticle);

		TSet<FPBDRigidParticleHandle*> ActivatedChildren;
		check(!ClusteredParticle->Disabled());
		if (MChildren.Contains(ClusteredParticle))
		{
			ActivatedChildren = ReleaseClusterParticles(MChildren[ClusteredParticle]);
		}
		return ActivatedChildren;
	}

	void FRigidClustering::ResetAllEvents()
	{
		ResetAllClusterBreakings();
		ResetAllClusterCrumblings();
		CrumbledSinceLastUpdate.Reset();
	}

	void FRigidClustering::TrackBreakingCollision(FPBDRigidClusteredParticleHandle* ClusteredParticle)
	{
		if (auto Rigid = ClusteredParticle->CastToRigidParticle())
		{
			Rigid->ParticleCollisions().VisitCollisions([this, Rigid](FPBDCollisionConstraint& Collision)
			{
				// Get a generic handle for the "other" particle
				uint8 OtherIdx = Collision.GetParticle0() == Rigid ? 1 : 0;

				// Make sure this collision actually includes the clustered particle
				if (!ensure(Collision.GetParticle(1 - OtherIdx) == Rigid))
				{
					return ECollisionVisitorResult::Continue;
				}

				FGeometryParticleHandle* OtherGeometry = Collision.GetParticle(OtherIdx);
				if (OtherGeometry == nullptr)
				{
					return ECollisionVisitorResult::Continue;
				}

				FPBDRigidParticleHandle* OtherRigid = OtherGeometry->CastToRigidParticle();
				if (OtherRigid == nullptr)
				{
					return ECollisionVisitorResult::Continue;
				}

				if (Collision.AccumulatedImpulse.SizeSquared() <= SMALL_NUMBER)
				{
					return ECollisionVisitorResult::Continue;
				}

				// Flip the impulse if we're restoring particle 0's momentum.
				// This is because by convention constraint impulses point from 1 to 0.
				const FVec3 CollisionImpulse = 
					(Collision.GetParticle0() == OtherRigid)
					? -Collision.AccumulatedImpulse
					: Collision.AccumulatedImpulse;

				FConstGenericParticleHandle Generic(OtherRigid);
				// Compute the angular impulse based on distance from the contact point to the CoM
				const FVec3 Location = Collision.CalculateWorldContactLocation();
				const Chaos::FVec3 AngularImpulse = Chaos::FVec3::CrossProduct(Location - Generic->PCom(), CollisionImpulse);

				// Compute impulse velocities
				const FVec3 ImpulseVelocity = Generic->InvM() * CollisionImpulse;

				const FMatrix33 OtherInvI = Utilities::ComputeWorldSpaceInertia(Generic->QCom(), Generic->ConditionedInvI());
				const FVec3 AngularImpulseVelocity = OtherInvI * AngularImpulse;

				// Update linear and angular impulses for the body, to be integrated next solve

				FMomentumRestoringData& MomentumRestoringData = MomentumRestoringDataByParticle.FindOrAdd(OtherRigid);
				MomentumRestoringData.V += ImpulseVelocity * RestoreBreakingMomentumPercent;
				MomentumRestoringData.W += AngularImpulseVelocity * RestoreBreakingMomentumPercent;

				return ECollisionVisitorResult::Continue;
			});
		}
	}

	void FRigidClustering::RestoreBreakingMomentum()
	{
		for (TPair<FPBDRigidParticleHandle*, FMomentumRestoringData> Pair : MomentumRestoringDataByParticle)
		{
			if (FPBDRigidParticleHandle* Particle = Pair.Key)
			{
				const FMomentumRestoringData& Data = Pair.Value;
				Particle->SetV(Particle->GetV() + Data.V);
				Particle->SetW(Particle->GetW() + Data.W);
			}
		}
	}

	void FRigidClustering::RemoveFromMomentumRestoringStructures(const FPBDRigidParticleHandle* ParticleToRemove)
	{
		MomentumRestoringDataByParticle.Remove(ParticleToRemove);
	}

	void FRigidClustering::SendBreakingEvent(FPBDRigidClusteredParticleHandle* ClusteredParticle, bool bFromCrumble)
	{
		// only emit break event if the proxy needs it 
		if (FGeometryCollectionPhysicsProxy* ConcreteProxy = GetConcreteProxy(ClusteredParticle))
		{
			const FSimulationParameters& SimParams = ConcreteProxy->GetSimParameters();
			if (SimParams.bGenerateBreakingData || SimParams.bGenerateGlobalBreakingData)
			{
				FBreakingData& ClusterBreak = MAllClusterBreakings.AddDefaulted_GetRef();
				ClusterBreak.Proxy = ClusteredParticle->PhysicsProxy();
				ClusterBreak.Location = ClusteredParticle->GetX();
				ClusterBreak.Orientation = ClusteredParticle->GetR();
				ClusterBreak.Velocity = ClusteredParticle->GetV();
				ClusterBreak.AngularVelocity = ClusteredParticle->GetW();
				ClusterBreak.Mass = ClusteredParticle->M();
				if (ClusteredParticle->GetGeometry() && ClusteredParticle->GetGeometry()->HasBoundingBox())
				{
					ClusterBreak.BoundingBox = ClusteredParticle->GetGeometry()->BoundingBox();
				}
				ClusterBreak.TransformGroupIndex = ConcreteProxy->GetTransformGroupIndexFromHandle(ClusteredParticle);
				ClusterBreak.bFromCrumble = bFromCrumble;
				ClusterBreak.SetEmitterFlag(SimParams.bGenerateBreakingData, SimParams.bGenerateGlobalBreakingData);
			}
		}
	}

	
	void FRigidClustering::SendCrumblingEvent(FPBDRigidClusteredParticleHandle* ClusteredParticle)
	{
		// only emit crumble events if the proxy needs it 
		if (FGeometryCollectionPhysicsProxy* ConcreteProxy = GetConcreteProxy(ClusteredParticle))
		{
			const FSimulationParameters& SimParams = ConcreteProxy->GetSimParameters(); 
			if (SimParams.bGenerateCrumblingData || SimParams.bGenerateGlobalCrumblingData)
			{
				FCrumblingData& ClusterCrumbling = MAllClusterCrumblings.AddDefaulted_GetRef();
				ClusterCrumbling.Proxy = ClusteredParticle->PhysicsProxy();
				ClusterCrumbling.Location = ClusteredParticle->GetX();
				ClusterCrumbling.Orientation = ClusteredParticle->GetR();
				ClusterCrumbling.LinearVelocity = ClusteredParticle->GetV();
				ClusterCrumbling.AngularVelocity = ClusteredParticle->GetW();
				ClusterCrumbling.Mass = ClusteredParticle->M();
				ClusterCrumbling.SetEmitterFlag(SimParams.bGenerateCrumblingData, SimParams.bGenerateGlobalCrumblingData);
				if (ClusteredParticle->GetGeometry() && ClusteredParticle->GetGeometry()->HasBoundingBox())
				{
					ClusterCrumbling.LocalBounds = ClusteredParticle->GetGeometry()->BoundingBox();
				}
				if (SimParams.bGenerateCrumblingChildrenData || SimParams.bGenerateGlobalCrumblingChildrenData)
				{
					// when sending this event, children are still attached
					if (const FRigidHandleArray* Children = MChildren.Find(ClusteredParticle))
					{
						ConcreteProxy->GetTransformGroupIndicesFromHandles(*Children, ClusterCrumbling.Children);
					}
				}
			}
		}
	}

	TArray<FRigidClustering::FParticleIsland> FRigidClustering::FindIslandsInChildren(const FPBDRigidClusteredParticleHandle* ClusteredParticle, bool bTraverseInterclusterEdges)
	{
		const TArray<FPBDRigidParticleHandle*>& Children = MChildren[ClusteredParticle];
		
		TArray<FParticleIsland> Islands;

		// traverse connectivity and see how many connected pieces we have
		TSet<FPBDRigidParticleHandle*> ProcessedChildren;
		ProcessedChildren.Reserve(Children.Num());

		for (FPBDRigidParticleHandle* Child : Children)
		{
			if (ProcessedChildren.Contains(Child))
			{
				continue;
			}

			TArray<FPBDRigidParticleHandle*>& Island = Islands.AddDefaulted_GetRef();

			TArray<FPBDRigidParticleHandle*> ProcessingQueue;
			ProcessingQueue.Add(Child);
			ProcessedChildren.Add(Child);

			while (ProcessingQueue.Num())
			{
				if (FPBDRigidParticleHandle* ChildToProcess = ProcessingQueue.Pop())
				{
					Island.Add(ChildToProcess);
					for (const TConnectivityEdge<FReal>& Edge : ChildToProcess->CastToClustered()->ConnectivityEdges())
					{
						FPBDRigidParticleHandle* Sibling = Edge.Sibling;
						if (IsInterclusterEdge(*ChildToProcess, Edge))
						{
							if (!bTraverseInterclusterEdges)
							{
								continue;
							}

							// Intercluster edges need to find the parent particle that's actually a child of the input ClusteredParticle
							while (Sibling)
							{
								if (FPBDRigidClusteredParticleHandle* ClusterSibling = Sibling->CastToClustered())
								{
									if (ClusterSibling->Parent() == ClusteredParticle)
									{
										break;
									}

									Sibling = ClusterSibling->Parent();
								}
								else
								{
									Sibling = nullptr;
									break;
								}
							}
						}

						if (Sibling && !ProcessedChildren.Contains(Sibling))
						{
							ProcessingQueue.Add(Sibling);
							ProcessedChildren.Add(Sibling);
						}
					}
				}
			}
		}

		return Islands;
	}

	void FRigidClustering::RemoveChildFromParent(FPBDRigidParticleHandle* Child, FPBDRigidClusteredParticleHandle* ClusteredParent)
	{
		if (ensure(Child != nullptr && ClusteredParent != nullptr))
		{
			FPBDRigidClusteredParticleHandle* ClusteredChild = Child->CastToClustered();
			if (!ClusteredChild || ClusteredChild->Parent() != ClusteredParent)
			{
				return;
			}

			MEvolution.EnableParticle(Child);
			TopLevelClusterParents.Add(ClusteredChild);

			ClusteredChild->SetClusterId(ClusterId(nullptr, ClusteredChild->ClusterIds().NumChildren)); // clear Id but retain number of children

			const FRigidTransform3 PreSolveTM(ClusteredParent->GetP(), ClusteredParent->GetQ());
			const FRigidTransform3 ChildFrame = ClusteredChild->ChildToParent() * PreSolveTM;
			Child->SetX(ChildFrame.GetTranslation());
			Child->SetR(ChildFrame.GetRotation());

			Child->SetP(Child->GetX());
			Child->SetQf(Child->GetRf());

			//todo(ocohen): in the presence of collisions, this will leave all children with the post-collision
			// velocity. This should be controlled by material properties so we can allow the broken pieces to
			// maintain the clusters pre-collision velocity.
			const FVec3f PDiff = Child->GetP() - ClusteredParent->GetP();
			Child->SetVf(Child->GetVf() + ClusteredParent->GetVf() + FVec3f::CrossProduct(ClusteredParent->GetWf(), PDiff));
			Child->SetWf(Child->GetWf() + ClusteredParent->GetWf());
			Child->SetPreVf(Child->GetPreVf() + ClusteredParent->GetPreVf() + FVec3f::CrossProduct(ClusteredParent->GetPreWf(), PDiff));
			Child->SetPreWf(Child->GetPreWf() + ClusteredParent->GetPreWf());

			// We also need to do cluster book-keeping on the parent.
			// If the parent is an internal cluster and has become empty, we need to mark this particle as ready to be destroyed.
			// The only exception is for cluster unions which are managed separately.
			if (FRigidHandleArray* Children = MChildren.Find(ClusteredParent))
			{
				ClusteredParent->SetClusterId(ClusterId{ ClusteredParent->Parent(), Children->Num()});
				if (ClusteredParent->InternalCluster() && Children->IsEmpty() && ClusteredParent->PhysicsProxy() && ClusteredParent->PhysicsProxy()->GetType() == EPhysicsProxyType::GeometryCollectionType)
				{
					// It's safe to disable the particle until we get to the point where we want to destroy the particle.
					DisableParticle(ClusteredParent);

					// We shouldn't ever need to do an AddUnique here since when we remove a child from a parent, the parent should only ever turn empty once.
					EmptyInternalClustersPerProxy.FindOrAdd(ClusteredParent->PhysicsProxy()).Add(ClusteredParent);
				}
			}
		}
	}

	void FRigidClustering::RemoveChildFromParentAndChildrenArray(FPBDRigidParticleHandle* Child, FPBDRigidClusteredParticleHandle* ClusteredParent)
	{
		// Also need to remove it from the children array.
		if (FRigidHandleArray* Children = MChildren.Find(ClusteredParent))
		{
			const int32 Index = Children->Find(Child);
			if (Index != INDEX_NONE)
			{
				Children->RemoveAtSwap(Index, EAllowShrinking::No);
			}
		}

		RemoveChildFromParent(Child, ClusteredParent);
	}

	TArray<FPBDRigidParticleHandle*> FRigidClustering::CreateClustersFromNewIslands(
		TArray<FParticleIsland>& Islands,
		FPBDRigidClusteredParticleHandle* ClusteredParent
		)
	{
		TArray<FPBDRigidParticleHandle*> NewClusters;
		
		// only for island with more than one particle
		int32 NumNewClusters = 0;
		for (const TArray<FPBDRigidParticleHandle*>& Island : Islands)
		{
			if (Island.Num() > 1)
			{
				NumNewClusters++;
			}
		}
		NewClusters.Reserve(NumNewClusters);

		const FRigidTransform3 PreSolveTM = FRigidTransform3(ClusteredParent->GetP(), ClusteredParent->GetQ());
		
		TArray<Chaos::FPBDRigidClusteredParticleHandle*> NewClusterHandles = MEvolution.CreateClusteredParticles(NumNewClusters);
		int32 ClusterHandlesIdx = 0;
		for (TArray<FPBDRigidParticleHandle*>& Island : Islands)
		{
			if (Island.Num() > 1) //now build the remaining pieces
			{
				FClusterCreationParameters CreationParameters;
				CreationParameters.ClusterParticleHandle = NewClusterHandles[ClusterHandlesIdx++];
				Chaos::FPBDRigidClusteredParticleHandle* NewCluster = 
					CreateClusterParticleFromClusterChildren(
						MoveTemp(Island), 
						ClusteredParent, 
						PreSolveTM, 
						CreationParameters);

				MEvolution.SetPhysicsMaterial(NewCluster, MEvolution.GetPhysicsMaterial(ClusteredParent));

				NewCluster->SetInternalStrains(ClusteredParent->GetInternalStrains());
				NewCluster->SetVf(ClusteredParent->GetVf());
				NewCluster->SetWf(ClusteredParent->GetWf());
				NewCluster->SetPreVf(ClusteredParent->GetPreVf());
				NewCluster->SetPreWf(ClusteredParent->GetPreWf());
				NewCluster->SetP(NewCluster->GetX());
				NewCluster->SetQf(NewCluster->GetRf());

				UpdateTopLevelParticle(NewCluster);

				// Need to get the material from the previous particle and apply it to the new one
				const FShapesArray& ChildShapes = ClusteredParent->ShapesArray();
				const FShapesArray& NewShapes = NewCluster->ShapesArray();
				const int32 NumChildShapes = ClusteredParent->ShapesArray().Num();

				if(NumChildShapes > 0)
				{
					// Can only take materials if the child has any - otherwise we fall back on defaults.
					// Due to GC initialisation however, we should always have a valid material as even
					// when one cannot be found we fall back on the default on GEngine
					const int32 NumChildMaterials = ChildShapes[0]->NumMaterials();
					if(NumChildMaterials > 0)
					{
						Chaos::FMaterialHandle ChildMat = ChildShapes[0]->GetMaterial(0);

						for(const TUniquePtr<FPerShapeData>& PerShape : NewShapes)
						{
							PerShape->SetMaterial(ChildMat);
						}
					}
				}
				NewClusters.Add(NewCluster);
			}
		}
		return NewClusters;
	}

	void FRigidClustering::SetInternalStrain(FPBDRigidClusteredParticleHandle* Particle, FRealSingle Strain)
	{
		Particle->SetInternalStrains(Strain);
		UpdateTopLevelParticle(Particle);
	}

	void FRigidClustering::SetExternalStrain(FPBDRigidClusteredParticleHandle* Particle, FRealSingle Strain)
	{
		Particle->SetExternalStrains(Strain);
		UpdateTopLevelParticle(Particle);
	}

	void FRigidClustering::UpdateTopLevelParticle(FPBDRigidClusteredParticleHandle* Particle)
	{
		FPBDRigidClusteredParticleHandle* ParticleToAdd = Particle;
		FPBDRigidClusteredParticleHandle* Parent = Particle->Parent();
		if (Parent != nullptr)
		{
			ParticleToAdd = Parent;
		}
		// make sure we only update the timestamp if it is not already in the map
		TopLevelClusterParentsStrained.FindOrAdd(ParticleToAdd, FPlatformTime::Cycles());
	}
	
	DECLARE_CYCLE_STAT(TEXT("TPBDRigidClustering<>::ReleaseClusterParticles(STRAIN)"), STAT_ReleaseClusterParticles_STRAIN, STATGROUP_Chaos);
	TSet<FPBDRigidParticleHandle*> FRigidClustering::ReleaseClusterParticles(
		FPBDRigidClusteredParticleHandle* ClusteredParticle,
		bool bForceRelease)
	{
		SCOPE_CYCLE_COUNTER(STAT_ReleaseClusterParticles_STRAIN);

		if (FPBDRigidClusteredParticleHandle* Parent = ClusteredParticle->Parent())
		{
			// Having a parent is only OK if the parent is a cluster union since ReleaseClusterParticlesImpl will
			// cause it to be ejected from the cluster union.
			if (!ensureMsgf((ClusterUnionManager.FindClusterUnionIndexFromParticle(Parent) != INDEX_NONE), TEXT("Removing a cluster that still has a non-cluster union parent")))
			{
				TSet<FPBDRigidParticleHandle*> EmptySet;
				return EmptySet;
			}
		}

		return ReleaseClusterParticlesImpl(ClusteredParticle, bForceRelease, true /*bCreateNewClusters*/);
	}
	
	DECLARE_CYCLE_STAT(TEXT("TPBDRigidClustering<>::ReleaseClusterParticlesNoInternalCluster"), STAT_ReleaseClusterParticlesNoInternalCluster, STATGROUP_Chaos);
	TSet<FPBDRigidParticleHandle*> FRigidClustering::ReleaseClusterParticlesNoInternalCluster(
		FPBDRigidClusteredParticleHandle* ClusteredParticle,
		bool bForceRelease)
	{
		/* This is a near duplicate of the ReleaseClusterParticles() method with the internal cluster creation removed.
		*  This method should be used exclusively by the GeometryCollectionComponentCacheAdaptor in order to implement
		*  correct behavior when cluster grouping is used. 
		*/
		
		SCOPE_CYCLE_COUNTER(STAT_ReleaseClusterParticlesNoInternalCluster);

		return ReleaseClusterParticlesImpl(ClusteredParticle, bForceRelease, false /*bCreateNewClusters*/);
	}

	TSet<FPBDRigidParticleHandle*> FRigidClustering::ReleaseClusterParticlesImpl(
		FPBDRigidClusteredParticleHandle* ClusteredParticle,
		bool bForceRelease,
		bool bCreateNewClusters)
	{	
		TSet<FPBDRigidParticleHandle*> ActivatedChildren;

		if (ClusteredParticle && ClusteredParticle->Unbreakable())
		{
			return ActivatedChildren;
		}

		if (!ensureMsgf(MChildren.Contains(ClusteredParticle), TEXT("Removing Cluster that does not exist!")))
		{
			return ActivatedChildren;
		}

		if (CVars::bChaosEnableOverrideSingleParticleTypeToCluster)
		{
			// early out so components that are added from static and skeletal meshes that rely on SingleParticlePhysicsProxy will not fall to pieces immediately!
			// TODO: This means they are currently unbreakable which is currently ok, but we may need them to have damage in the future
			return ActivatedChildren;
		}

		// gather propagation information from the parent proxy
		bool bUseDamagePropagation = false;
		float BreakDamagePropagationFactor = 0.0f;
		float ShockDamagePropagationFactor = 0.0f;
		FGeometryCollectionPhysicsProxy* ConcreteGCProxy = GetConcreteProxy<FGeometryCollectionPhysicsProxy>(ClusteredParticle);
		if (ConcreteGCProxy)
		{
			const FSimulationParameters& SimParams = ConcreteGCProxy->GetSimParameters();
			bUseDamagePropagation = SimParams.bUseDamagePropagation;
			BreakDamagePropagationFactor = SimParams.BreakDamagePropagationFactor;
			ShockDamagePropagationFactor = SimParams.ShockDamagePropagationFactor;
			
			if (GeometryCollectionPhysicsProxy::Private::BuildGeometryForChildrenOnPT(SimParams) == false)
			{
				ConcreteGCProxy->CreateChildrenGeometry_Internal();
			}
		}

		TArray<FPBDRigidParticleHandle*>& Children = MChildren[ClusteredParticle];
		const bool bParentCrumbled = CrumbledSinceLastUpdate.Contains(ClusteredParticle);

		bool bFoundFirstRelease = false;

		// only used for propagation
		TMap<FPBDRigidParticleHandle*, FRealSingle> AppliedStrains;

		// We'll pass these particles to the cluster union manager to remove. This can't be done within the same loop
		// since it'll be modifying the children array.
		TArray<FPBDRigidParticleHandle*> DeferredRemoveFromClusterUnion;

		// Grab cluster union parent if there is one
		FPBDRigidParticleHandle* ParentRigid = ClusteredParticle->ClusterIds().Id;
		FPBDRigidClusteredParticleHandle* Parent = ParentRigid ? ParentRigid->CastToClustered() : nullptr;
		TSet<FClusterUnionIndex> ClusterUnionsToConsiderForConnectivity;

		for (int32 ChildIdx = Children.Num() - 1; ChildIdx >= 0; --ChildIdx)
		{
			FPBDRigidClusteredParticleHandle* Child = Children[ChildIdx]->CastToClustered();
			
			if (!Child)
			{
				continue;
			}

			// @todo(chaos) eventually should get rid of collision impulse array and only use external strain
			const FRealSingle MaxAppliedStrain = FMath::Max(Child->CollisionImpulses(), Child->GetExternalStrain());
			if ((MaxAppliedStrain >= Child->GetInternalStrains()) || bForceRelease)
			{
				if (!bFoundFirstRelease)
				{
					// Restore some of the momentum of whatever collided with the parent
					// NOTE: This has to come before HandleRemoveOperationWithClusterLookup, because
					// in FClusterUnionManager::UpdateAllClusterUnionProperties, the particle is
					// invalidated with MEvolution.InvalidateParticle, which clears its contacts
					if (RestoreBreakingMomentumPercent > 0.f)
					{
						if (Parent)
						{
							TrackBreakingCollision(Parent);
						}
						else
						{
							TrackBreakingCollision(ClusteredParticle);
						}
					}

					const FClusterUnionIndex ClusterUnionIndex = ClusterUnionManager.FindClusterUnionIndexFromParticle(ClusteredParticle);
					if (ClusterUnionIndex != INDEX_NONE)
					{
						// Generating intercluster edges depends on the parent particle's node connections existing. Thus this must go before RemoveNodeConnections.
						// Our parent particle is being removed from the cluster union - initialize intercluster edges if necessary.
						if (FClusterUnion* ClusterUnion = ClusterUnionManager.FindClusterUnion(ClusterUnionIndex))
						{
							ClusterUnionManager.GenerateInterclusterEdgesForParticle(*ClusterUnion, ClusteredParticle);
						}

						// Remove node connections here immediately just in case we need to manage connectivity on the cluster union.
						RemoveNodeConnections(ClusteredParticle);
						ClusterUnionsToConsiderForConnectivity.Add(ClusterUnionIndex);

						ClusterUnionManager.HandleRemoveOperationWithClusterLookup({ ClusteredParticle }, EClusterUnionOperationTiming::Defer);
					}
					bFoundFirstRelease = true;
				}

				// There's a possibility that the child is in a cluster union so we'd need to be able to remove the child particle from the cluster union as well.
				const FClusterUnionIndex ClusterUnionIndex = ClusterUnionManager.FindClusterUnionIndexFromParticle(Child);
				const bool bIsInClusterUnion = ClusterUnionIndex != INDEX_NONE;

				if (bIsInClusterUnion)
				{
					ClusterUnionsToConsiderForConnectivity.Add(ClusterUnionIndex);
					DeferredRemoveFromClusterUnion.Add(Child);
				}
				else
				{
					// The piece that hits just breaks off - we may want more control 
					// by looking at the edges of this piece which would give us cleaner 
					// breaks (this approach produces more rubble)
					RemoveChildFromParent(Child, ClusteredParticle);
					UpdateTopLevelParticle(Child);

					// Remove from the children array without freeing memory yet. 
					// We're looping over Children and it'd be silly to free the array
					// 1 entry at a time.
					Children.RemoveAtSwap(ChildIdx, EAllowShrinking::No);
				}
				ActivatedChildren.Add(Child);
				SendBreakingEvent(Child, bParentCrumbled);
			}
			if (bUseDamagePropagation)
			{
				AppliedStrains.Add(Child, MaxAppliedStrain);
			}
			Child->SetExternalStrains(0.0);
		}

		if (!DeferredRemoveFromClusterUnion.IsEmpty())
		{
			ClusterUnionManager.HandleRemoveOperationWithClusterLookup(DeferredRemoveFromClusterUnion, EClusterUnionOperationTiming::Defer);
		}

		// if necessary propagate strain through the graph
		// IMPORTANT: this assumes that the connectivity graph has not yet been updated from pieces that broke off
		if (bUseDamagePropagation)
		{
			for (const auto& AppliedStrain: AppliedStrains)
			{
				FPBDRigidClusteredParticleHandle* ClusteredChild = AppliedStrain.Key->CastToClustered();

				const FRealSingle AppliedStrainValue = AppliedStrain.Value;
				FRealSingle PropagatedStrainPerConnection = 0.0f;

				// @todo(chaos) : may not be optimal, but good enough for now
				if (BreakDamagePropagationFactor > 0 && ActivatedChildren.Contains(AppliedStrain.Key))
				{
					// break damage propagation case: we only look at the broken pieces and propagate the strain remainder 
					const FRealSingle RemainingStrain = (AppliedStrainValue - ClusteredChild->GetInternalStrains());
					if (RemainingStrain > 0)
					{
						const FRealSingle AdjustedRemainingStrain = BreakDamagePropagationFactor * RemainingStrain;
						// todo(chaos) : could do better and have something weighted on distance with a falloff maybe ?  
						PropagatedStrainPerConnection = AdjustedRemainingStrain / static_cast<FRealSingle>(ClusteredChild->ConnectivityEdges().Num());
					}
				}
				else if (ShockDamagePropagationFactor > 0)
				{
					// shock damage propagation case : for all the non broken pieces, propagate the actual applied strain 
					PropagatedStrainPerConnection = ShockDamagePropagationFactor * AppliedStrainValue;
				}

				if (PropagatedStrainPerConnection > 0)
				{
					for (const TConnectivityEdge<FReal>& Edge : ClusteredChild->ConnectivityEdges())
					{
						if (Edge.Sibling)
						{
							if (FPBDRigidClusteredParticleHandle* ClusteredSibling = Edge.Sibling->CastToClustered())
							{
								// todo(chaos) this may currently be non optimal as we are in the apply loop and this may be cleared right after
								SetExternalStrain(ClusteredSibling, FMath::Max(ClusteredSibling->GetExternalStrain(), PropagatedStrainPerConnection));
							}
						}
					}
				}
			}
		}

		if (ActivatedChildren.Num() > 0)
		{
			const bool bIsClusterUnion = ClusterUnionManager.IsClusterUnionParticle(ClusteredParticle);
			if (Children.Num() == 0)
			{
				// Free the memory if we can do so cheaply (no data copies).
				Children.Empty(); 
			}

			if (UseConnectivity)
			{
				// The cluster may have contained forests, so find the connected pieces and cluster them together.
				//first update the connected graph of the children we already removed
				for (FPBDRigidParticleHandle* Child : ActivatedChildren)
				{
					RemoveNodeConnections(Child);
				}
				ActivatedChildren.Append(HandleConnectivityOnReleaseClusterParticle(ClusteredParticle, bCreateNewClusters));
			}

			for (FPBDRigidParticleHandle* Child : ActivatedChildren)
			{
				// If an activated child has a parent, we don't want to update their kinematic properties since they should be disabled
				// and thus, shouldn't need to have properties updated.
				if (FPBDRigidClusteredParticleHandle* ClusteredChild = Child->CastToClustered())
				{
					if (ClusteredChild->Parent())
					{
						check(ClusteredChild->Disabled());
						continue;
					}
				}
				UpdateKinematicProperties(Child, MChildren, MEvolution);
			}

			// Disable the cluster only if we're not a cluster union. Cluster unions will handle themselves separately.
			if (!bIsClusterUnion)
			{
				DisableCluster(ClusteredParticle);
			}
		}

		// optimization : start disabling activated children if the number of active released particle is too high
		ThrottleReleasedParticlesIfNecessary(ActivatedChildren);

		FrameReleasedChildren += ActivatedChildren.Num();

		return ActivatedChildren;
	}

	DECLARE_CYCLE_STAT(TEXT("TPBDRigidClustering<>::HandleConnectivityOnReleaseClusterParticle"), HandleConnectivityOnReleaseClusterParticle, STATGROUP_Chaos);
	TSet<FPBDRigidParticleHandle*> FRigidClustering::HandleConnectivityOnReleaseClusterParticle(FPBDRigidClusteredParticleHandle* ClusteredParticle, bool bCreateNewClusters)
	{
		SCOPE_CYCLE_COUNTER(HandleConnectivityOnReleaseClusterParticle);
		if (!ensure(ClusteredParticle))
		{
			return {};
		}

		TSet<FPBDRigidParticleHandle*> ActivatedChildren;
		const bool bHasChildren = (ClusteredParticle->ClusterIds().NumChildren > 0);

		// If we're breaking a geometry collection, we'll need to create internal clusters to parent the remaining particles.
		// However, we do not need to do this if we're currently operating on a cluster union! Its remaining particles should stay
		// attached to the cluster union because they can handle particles being dynamically added/removed.
		const FClusterUnion* ParentClusterUnion = ClusterUnionManager.FindClusterUnionFromParticle(ClusteredParticle);
		const bool bIsClusterUnion = ParentClusterUnion != nullptr;
		if (bHasChildren)
		{
			TArray<FParticleIsland> Islands = FindIslandsInChildren(ClusteredParticle, bIsClusterUnion);

			if (!bIsClusterUnion)
			{
				TArray<const FConnectivityEdge*> IslandInterclusterEdges;
				IslandInterclusterEdges.Reserve(Islands.Num());

				// By default, all these islands will all just start simulating independently. However, if bCheckForInterclusterEdgesOnRelease
				// is true, we're going to want to check if any of these islands contain a particle with an intercluster edge that connects to
				// something else! In that case, keep the particles in the island connected. For simplicity
				// we'll use a kinematic target for each group of particles that needs to stay attached . This kinematic target
				// will be driven by simulation relative to whatever it should be attached to.
				//
				// We need to assume that only the server will have the intercluster edges necessary to drive this kinematic target. Therefore, this
				// information is authoritative on the PT on the server while is authoritative on the GT on the client(s) via GC replication. So we
				// also need to make sure we store this information in a way that's replicatable to the client via GC replication.
				EDamageEvaluationModel DamageEvaluationModel = EDamageEvaluationModel::StrainFromDamageThreshold;
				const FGeometryCollectionPhysicsProxy* ConcreteGCProxy = GetConcreteProxy<FGeometryCollectionPhysicsProxy>(ClusteredParticle);
				if (ConcreteGCProxy)
				{
					const FSimulationParameters& SimParams = ConcreteGCProxy->GetSimParameters();
					DamageEvaluationModel = SimParams.DamageEvaluationModel;
				}

				int32 IslandIndex = 0;
				TArray<int32> IslandIndicesToRemove;

				for (const FParticleIsland& Island : Islands)
				{
					const bool bNeedRecomputeConnectivityStrain = ConcreteGCProxy && DamageEvaluationModel == EDamageEvaluationModel::StrainFromMaterialStrengthAndConnectivity;

					FClusterUnion* AttachedClusterUnion = nullptr;

					// Protect a potentially work-intensive loop behind the conditions that actually require us to step through every child in every island.
					if (bNeedRecomputeConnectivityStrain || bCheckForInterclusterEdgesOnRelease)
					{
						for (FPBDRigidParticleHandle* ChildParticle : Island)
						{
							if (FPBDRigidClusteredParticleHandle* ClusteredChild = ChildParticle->CastToClustered())
							{
								// recompute the strain as its connectivity has changed
								if (bNeedRecomputeConnectivityStrain)
								{
									ConcreteGCProxy->ComputeMaterialBasedDamageThreshold_Internal(*ClusteredChild);
								}

								if (bCheckForInterclusterEdgesOnRelease && !AttachedClusterUnion)
								{
									// Check if any edge is connected to "something else". This entire island
									// will be connected to that "something else". For now, we're enforcing that it's a cluster union.
									for (const FConnectivityEdge& Edge : ClusteredChild->ConnectivityEdges())
									{
										if (IsInterclusterEdge(*ClusteredChild, Edge))
										{
											if (FClusterUnion* ClusterUnion = ClusterUnionManager.FindClusterUnionFromParticle(GetActiveParentParticle(Edge.Sibling)))
											{
												if (ClusterUnion->bCheckConnectivity)
												{
													AttachedClusterUnion = ClusterUnion;
													break;
												}
											}
										}
									}
								}
							}
						}
					}

					if (AttachedClusterUnion)
					{
						// We may not actually want to attach the *entire* island into the cluster union. 
						// We may only want to attach a subset into the cluster union. In that case, we just release
						// each of the remaining particles separately (TODO: maybe figure out a way to create internal clusters for them).
						TArray<FPBDRigidParticleHandle*> ParticlesForClusterUnion;
						TArray<FPBDRigidParticleHandle*> ParticlesToRelease;

						if (bOnlyUseInterclusterEdgesAttachedToMainParticles)
						{
							ParticlesForClusterUnion.Reserve(Island.Num());
							ParticlesToRelease.Reserve(Island.Num());
						}

						for (FPBDRigidParticleHandle* ChildParticle : Island)
						{
							if (bOnlyUseInterclusterEdgesAttachedToMainParticles)
							{
								if (ClusterUnionManager.IsDirectlyConnectedToMainParticleInClusterUnion(*AttachedClusterUnion, ChildParticle))
								{
									ParticlesForClusterUnion.Add(ChildParticle);
								}
								else
								{
									ParticlesToRelease.Add(ChildParticle);
								}
							}

							// Need to manually add cluster union properties since the API for adding a pending operation doesn't have an option for that.
							FClusterUnionParticleProperties Properties;
							Properties.bIsAuxiliaryParticle = true;
							AttachedClusterUnion->ChildProperties.Add(ChildParticle, Properties);
						}
						ClusterUnionManager.AddPendingClusterIndexOperation(AttachedClusterUnion->InternalIndex, EClusterUnionOperation::Add, bOnlyUseInterclusterEdgesAttachedToMainParticles ? ParticlesForClusterUnion : Island);

						if (!ParticlesToRelease.IsEmpty())
						{
							for (FPBDRigidParticleHandle* ChildParticle : ParticlesToRelease)
							{
								// Need to remove node connections here. Otherwise it may be possible for the cluster union to have erroneous intercluster edges that connect it to another cluster union.
								RemoveNodeConnections(ChildParticle);
							}
							RemoveParticlesFromCluster(ClusteredParticle, ParticlesToRelease);
							ActivatedChildren.Append(ParticlesToRelease);
						}
						IslandIndicesToRemove.Add(IslandIndex);
					}
					else if (Island.Num() == 1 && !AttachedClusterUnion) //need to break single pieces first
					{
						FPBDRigidParticleHandle* Child = Island[0];
						RemoveParticlesFromCluster(ClusteredParticle, { Child });
						ActivatedChildren.Add(Child);
					}

					++IslandIndex;
				}

				for (int32 RemoveIndex = IslandIndicesToRemove.Num() - 1; RemoveIndex >= 0; --RemoveIndex)
				{
					Islands.RemoveAtSwap(IslandIndicesToRemove[RemoveIndex], EAllowShrinking::No);
				}

				if (bCreateNewClusters)
				{
					// Each island is going to be removed from the parent particle. Pre-emptively remove each island from the parent particle's book-keeping.
					// If we don't do this, the particles will still be stored as children of the particle's previous parents in MChildren.
					for (const FParticleIsland& Island : Islands)
					{
						RemoveParticlesFromCluster(ClusteredParticle, Island);

						if (Island.Num() > 1)
						{
							// Need to subsequently disable the particle because they probably
							// just got re-enabled in RemoveParticlesFromCluster.
							for (FPBDRigidParticleHandle* Particle : Island)
							{
								// Particles may be re-enabled again at a later stage  so we do not want to remove them from the children map 
								DisableParticle(Particle, /*bRemoveFromChildrenMap*/ false);
							}
						}
					}

					TArray<FPBDRigidParticleHandle*> NewClusters = CreateClustersFromNewIslands(Islands, ClusteredParticle);
					ActivatedChildren.Append(MoveTemp(NewClusters));
				}
			}
			else if (bCheckForInterclusterEdgesOnRelease && ParentClusterUnion->bCheckConnectivity)
			{
				// We know we're in an cluster union. There are pieces that we consider to be the "main body". There are certain pieces that we consider to be auxiliary as well.
				// If an island is only made up of auxiliary pieces, then those pieces should fall off. Connectivity of main pieces should be handled by the GT.
				for (const FParticleIsland& Island : Islands)
				{
					bool bHasMainParticle = false;
					
					TArray<FPBDRigidParticleHandle*> ParticlesToRemove;
					for (FPBDRigidParticleHandle* ChildParticle : Island)
					{
						if (!bHasMainParticle)
						{
							if (const FClusterUnionParticleProperties* Props = ParentClusterUnion->ChildProperties.Find(ChildParticle))
							{
								if (!Props->bIsAuxiliaryParticle)
								{
									bHasMainParticle = true;
								}
							}
						}
						
						if (bOnlyUseInterclusterEdgesAttachedToMainParticles)
						{
							if (!ClusterUnionManager.IsDirectlyConnectedToMainParticleInClusterUnion(*ParentClusterUnion, ChildParticle))
							{
								ParticlesToRemove.Add(ChildParticle);
							}
						}
						else if (bHasMainParticle)
						{
							break;
						}
					}

					if (!bHasMainParticle)
					{
						ClusterUnionManager.AddPendingClusterIndexOperation(ParentClusterUnion->InternalIndex, EClusterUnionOperation::Remove, Island);
						ActivatedChildren.Append(Island);
					}
					else if (!ParticlesToRemove.IsEmpty())
					{
						ClusterUnionManager.AddPendingClusterIndexOperation(ParentClusterUnion->InternalIndex, EClusterUnionOperation::Remove, ParticlesToRemove);
						ActivatedChildren.Append(ParticlesToRemove);
					}
				}
			}
		}

		return ActivatedChildren;
	}

	DECLARE_CYCLE_STAT(TEXT("TPBDRigidClustering<>::ReleaseClusterParticles(LIST)"), STAT_ReleaseClusterParticles_LIST, STATGROUP_Chaos);
	TSet<FPBDRigidParticleHandle*> 
	FRigidClustering::ReleaseClusterParticles(
		TArray<FPBDRigidParticleHandle*> ChildrenParticles, bool bTriggerBreakEvents /* = false */)
	{
		SCOPE_CYCLE_COUNTER(STAT_ReleaseClusterParticles_LIST);
		TSet<FPBDRigidParticleHandle*> ActivatedBodies;
		if (ChildrenParticles.Num())
		{
			//for now just assume these all belong to same cluster
			FPBDRigidParticleHandle* ClusterHandle = nullptr;
			
			TMap<FGeometryParticleHandle*, FReal> FakeStrain;

			bool bPreDoGenerateData = DoGenerateBreakingData;
			DoGenerateBreakingData = bTriggerBreakEvents;

			for (FPBDRigidParticleHandle* ChildHandle : ChildrenParticles)
			{
				if (FPBDRigidClusteredParticleHandle* ClusteredChildHandle = ChildHandle->CastToClustered())
				{
					if (ClusteredChildHandle->Disabled() && ClusteredChildHandle->ClusterIds().Id != nullptr)
					{
						if (ensure(!ClusterHandle || ClusteredChildHandle->ClusterIds().Id == ClusterHandle))
						{
							SetExternalStrain(ClusteredChildHandle, TNumericLimits<FRealSingle>::Max());
							// This way we won't try to propagate this infinite strain.
							ClusteredChildHandle->SetInternalStrains(ClusteredChildHandle->GetExternalStrain());
							ClusterHandle = ClusteredChildHandle->ClusterIds().Id;
						}
						else
						{
							break; //shouldn't be here
						}
					}
				}
			}
			if (ClusterHandle)
			{
				ActivatedBodies = ReleaseClusterParticles(ClusterHandle->CastToClustered());
			}
			DoGenerateBreakingData = bPreDoGenerateData;
		}
		return ActivatedBodies;
	}

	DECLARE_CYCLE_STAT(TEXT("TPBDRigidClustering<>::ReleaseChildrenParticleAndParents"), ReleaseChildrenParticleAndParents, STATGROUP_Chaos);
	void FRigidClustering::ForceReleaseChildParticleAndParents(FPBDRigidClusteredParticleHandle* ChildClusteredParticle, bool bTriggerBreakEvents)
	{
		SCOPE_CYCLE_COUNTER(ReleaseChildrenParticleAndParents);

		if (ChildClusteredParticle)
		{
			// make sure we set unbreakable to false so that the children can be released
			ChildClusteredParticle->SetUnbreakable(false);
			
			// first release any parent if any
			if (FPBDRigidClusteredParticleHandle* ParentCluster = ChildClusteredParticle->Parent())
			{
				if (const IPhysicsProxyBase* ParentProxy = ParentCluster->PhysicsProxy())
				{
					// we shoudl not break cluster union parent and stop recursion there
					if (ParentProxy->GetType() != FClusterUnionPhysicsProxy::ConcreteType())
					{
						// we need now to force parents to break
						ForceReleaseChildParticleAndParents(ParentCluster, bTriggerBreakEvents);
					}
				}
			}

			// Trigger a release.
			ReleaseClusterParticles(TArray<FPBDRigidParticleHandle*>{ ChildClusteredParticle }, bTriggerBreakEvents);
		}
	}

	static int32 GClusterBreakOnlyStrained = 1;
	FAutoConsoleVariableRef CVarBreakMode(TEXT("p.chaos.clustering.breakonlystrained"), GClusterBreakOnlyStrained, 
										  TEXT("If enabled we only process strained clusters for breaks, if disabled all clusters are traversed and checked"));

	static int32 GPerAdvanceBreaksAllowed = TNumericLimits<int32>::Max();
	FAutoConsoleVariableRef CVarPerAdvanceBreaksAllowed(TEXT("p.Chaos.Clustering.PerAdvanceBreaksAllowed"), GPerAdvanceBreaksAllowed,
		TEXT("Number of breaks allowed to occur for each invocation of AdvanceClustering"));

	static int32 GPerAdvanceBreaksRescheduleLimit = TNumericLimits<int32>::Max();
	FAutoConsoleVariableRef CVarPerAdvanceBreaksRescheduleLimit(TEXT("p.Chaos.Clustering.PerAdvanceBreaksRescheduleLimit"), GPerAdvanceBreaksRescheduleLimit,
		TEXT("Number of breaks allowed to be rescheduled for next frame if any "));

	static int32 GDumpClusterAndReleaseStats = 0;
	FAutoConsoleVariableRef CVarDumpClusterAndReleaseStats(TEXT("p.Chaos.Clustering.DumpClusterAndReleaseStats"), GDumpClusterAndReleaseStats,
		TEXT("Report the number of cluster processes and released particles per frame, on/off 1/0"));

	DECLARE_CYCLE_STAT(TEXT("TPBDRigidClustering<>::AdvanceClustering"), STAT_AdvanceClustering, STATGROUP_Chaos);
	DECLARE_CYCLE_STAT(TEXT("TPBDRigidClustering<>::Update Impulse from Strain"), STAT_UpdateImpulseStrain, STATGROUP_Chaos);
	DECLARE_CYCLE_STAT(TEXT("TPBDRigidClustering<>::Update Dirty Impulses"), STAT_UpdateDirtyImpulses, STATGROUP_Chaos);
	
	void FRigidClustering::AdvanceClustering(
		const FReal Dt, 
		FPBDCollisionConstraints& CollisionRule)
	{
		SCOPE_CYCLE_COUNTER(STAT_AdvanceClustering);
		UE_LOGF(LogChaos, Verbose, "START FRAME with Dt %f", Dt);

		double FrameTime = 0, Time = 0;
		FDurationTimer Timer(Time);
		Timer.Start();

		if(GDumpClusterAndReleaseStats == 1)
		{
			if(AdvanceCount > 0 && FrameReleasedChildren > 0)
			{
				UE_LOGF(LogChaos, Display, "Clustering | Frame %.5u | Clusters: %.3u, Released: %.4u (TotalClusters: %.3u, TotalReleased: %.4u)",
					   AdvanceCount,
					   FrameProcessedClusters,
					   FrameReleasedChildren,
					   TotalProcessedClusters,
					   TotalReleasedChildren);
			}
			AdvanceCount++;
			TotalProcessedClusters += FrameProcessedClusters;
			TotalReleasedChildren += FrameReleasedChildren;
		}
		FrameProcessedClusters = 0;
		FrameReleasedChildren = 0;

		if(MChildren.Num())
		{
			//
			//  Grab collision impulses for processing
			//
			if (ComputeClusterCollisionStrains)
			{
				ComputeStrainFromCollision(CollisionRule, Dt);
			}
			else
			{
				ResetCollisionImpulseArray();
			}

			//  Monitor the MStrain array for 0 or less values.
			//  That will trigger a break too.
			//
			bool bPotentialBreak = false;
			TArray<FPBDRigidClusteredParticleHandle*> ParticlesToProcess;

			auto ProcessClusteredParticle = [&ParticlesToProcess, &bPotentialBreak, this](FPBDRigidClusteredParticleHandle* Particle)
			{
				bool bAddParent = false;
				if (TArray<FRigidHandle>* ParentToChildren = MChildren.Find(Particle))
				{
					for (FRigidHandle Child : *ParentToChildren)
					{
						if (FClusterHandle ClusteredChild = Child->CastToClustered())
						{
							if (ClusteredChild->GetInternalStrains() <= 0.f)
							{
								bAddParent = true;
								// #TODO remove need to set this here so we can early out as soon as we
								// find one child that requires processing for breaks
								ClusteredChild->CollisionImpulse() = FLT_MAX;
								MCollisionImpulseArrayDirty = true;
							}
							else if (ClusteredChild->GetExternalStrain() > 0 || ClusteredChild->CollisionImpulse() > 0)
							{
								bAddParent = true;
								bPotentialBreak = true;
							}
						}
					}
				}
				else
				{
					ensureMsgf(false, TEXT("Could not find an entry in MChildrenMap for a cluster to process, this is not expected. Skipping the processing"));
				}

				// Ensure we only add the parent once.
				if(bAddParent)
				{
					ParticlesToProcess.Add(Particle);
				}
			};

			{
				SCOPE_CYCLE_COUNTER(STAT_UpdateDirtyImpulses);

				// sort by incrementing timestamp
				TopLevelClusterParentsStrained.ValueSort([](const int64 A, const int64 B) { return A < B; });

				// no process the strained parent and fill ParticlesToProcess 
				for (const auto& StrainedParentEntry: TopLevelClusterParentsStrained)
				{
					Chaos::FPBDRigidClusteredParticleHandle* ActiveCluster = StrainedParentEntry.Key;

					bool bIgnoreDisabledCheck = false;
					FClusterUnion* ClusterUnion = ClusterUnionManager.FindClusterUnionFromParticle(ActiveCluster);
					if (ClusterUnion && ClusterUnion->InternalCluster != ActiveCluster)
					{
						// Need to ignore the disabled check since the particle might still be inside a cluster union (and thus disabled).
						bIgnoreDisabledCheck = true;
					}

					if (!ActiveCluster->Disabled() || bIgnoreDisabledCheck)
					{
						if (ActiveCluster->ClusterIds().NumChildren > 0) //active index is a cluster
						{
							bool bNeedToProcessActiveCluster = true;

							if(ClusterUnion)
							{
								if(ClusterUnion->InternalCluster == ActiveCluster)
								{
									// ActiveCluster is itself a cluster union, so loop over its children and add those
									// to process for breaking.
									bNeedToProcessActiveCluster = false;
									for(FPBDRigidParticleHandle* ChildParticle : ClusterUnion->ChildParticles)
									{
										if(ChildParticle)
										{
											if(FPBDRigidClusteredParticleHandle* ClusteredChild = ChildParticle->CastToClustered())
											{
												const bool bChildHasChildren = ClusteredChild->ClusterIds().NumChildren > 0;

												// If for some reason, a child of the cluster union doesn't have children then we need to process the cluster union particle instead.
												bNeedToProcessActiveCluster |= !bChildHasChildren;

												if(bChildHasChildren)
												{
													ProcessClusteredParticle(ClusteredChild);
												}
											}
										}
									}
								}
							}
							
							if (bNeedToProcessActiveCluster)
							{
								ProcessClusteredParticle(ActiveCluster);
							}
						}
					}
				}
			}

			//
			// Modify internal strains. This needs to happen after the previous block so we can make sure
			// we're applying the strain modifiers on ParticlesToProcess instead of TopLevelClusterParentsStrained
			// since the latter may miss things related to cluster unions.
			//
			if (StrainModifiers)
			{
				ApplyStrainModifiers(ParticlesToProcess);
			}

			ClusterUnionManager.HandleDeferredClusterUnionUpdateProperties();

			// Breaking can populate this again with relevant children - so we clear before running the breaking model
			TopLevelClusterParentsStrained.Reset();

			if (MCollisionImpulseArrayDirty || bPotentialBreak)
			{
				SCOPE_CYCLE_COUNTER(STAT_UpdateDirtyImpulses);

				// Call our breaking model
				// #TODO convert to visitor pattern to avoid TArray allocations above.
				if(GClusterBreakOnlyStrained == 1)
				{
					// call break model for each particle and only count the ones we breaks
					// some may strained parent may result in non breaking clusters if strain modifier has changed the strain values
					// todo(chaos): we should certainly try to have the strain modifier providing a list of those instead of preemptively process thenm to realise that there's nothoing to break
					int32 NumBreaks = 0;
					int32 LastProcessedIndex = 0;
					for (int32 Index = 0; Index < ParticlesToProcess.Num(); Index++)
					{
						FPBDRigidClusteredParticleHandle* ParticleToProcess = ParticlesToProcess[Index];

						if (BreakingModel({ &ParticleToProcess , 1 }))
						{
							NumBreaks++;
							if (NumBreaks >= DestructionSettings.PerAdvanceBreaksAllowed)
							{
								LastProcessedIndex = Index;
								break;
							}
						}
					}

					// Add back the rest of the particles to process back in the strained array for later processing
					if (DestructionSettings.PerAdvanceBreaksRescheduleLimit > 0)
					{
						// Add back the non processed parent clusters for next tick
						for (int32 Index = 0; Index < DestructionSettings.PerAdvanceBreaksRescheduleLimit; Index++)
						{
							const int32 ParticleToProcessIndex = Index + LastProcessedIndex + 1;
							if (ParticleToProcessIndex >= ParticlesToProcess.Num())
							{
								break;
							}
							FPBDRigidClusteredParticleHandle* ParticleToProcessNextTick = ParticlesToProcess[ParticleToProcessIndex];
							// since ParticlesToProcess is sorted by time stamp, the new time stamp will make sure they are in the same order 
							TopLevelClusterParentsStrained.FindOrAdd(ParticleToProcessNextTick, FPlatformTime::Cycles());
						}
					}
				}
				else
				{
					BreakingModel();
				}

				if (bCheckForInterclusterEdgesOnRelease)
				{
					// In this case the breaking model might have tried to *add* into cluster unions so we need to flush those operations here as well
					// so those changes are immediately available for marshaling back to the GT.
					ClusterUnionManager.FlushPendingOperations();
				}

			} // end if MCollisionImpulseArrayDirty
		}

		Timer.Stop();
		UE_LOGF(LogChaos, Verbose, "Cluster Break Update Time is %f", Time);
	}

	DECLARE_CYCLE_STAT(TEXT("FRigidClustering::CleanupInternalClustersForProxy"), STAT_CleanupInternalClustersForProxy, STATGROUP_Chaos);
	void FRigidClustering::CleanupInternalClustersForProxies(TArrayView<IPhysicsProxyBase*> Proxies)
	{
		SCOPE_CYCLE_COUNTER(STAT_CleanupInternalClustersForProxy);
		for (IPhysicsProxyBase* Proxy : Proxies)
		{
			for (FPBDRigidClusteredParticleHandle* Particle : EmptyInternalClustersPerProxy.FindRef(Proxy))
			{
				DestroyClusterParticle(Particle);
			}
			EmptyInternalClustersPerProxy.Remove(Proxy);
		}
	}

	DECLARE_CYCLE_STAT(TEXT("BreakingModel_AllParticles"), STAT_BreakingModel_AllParticles, STATGROUP_Chaos);
	DECLARE_CYCLE_STAT(TEXT("BreakingModel"), STAT_BreakingModel, STATGROUP_Chaos);
	void FRigidClustering::BreakingModel()
	{
		SCOPE_CYCLE_COUNTER(STAT_BreakingModel_AllParticles);
		
		//make copy because release cluster modifies active indices. We want to iterate over original active indices
		TArray<FPBDRigidClusteredParticleHandle*> ClusteredParticlesToProcess;
		for(FTransientPBDRigidParticleHandle& Particle : MEvolution.GetNonDisabledClusteredView())
		{
			if (FPBDRigidClusteredParticleHandle* Clustered = Particle.Handle()->CastToClustered())
			{
				if (Clustered->ClusterIds().NumChildren > 0)
				{
					if (FClusterUnion* ClusterUnion = ClusterUnionManager.FindClusterUnionFromParticle(Clustered))
					{
						if (ClusterUnion->InternalCluster == Clustered)
						{
							// Clustered is itself a cluster union, so loop over its children and add those
							// to process for breaking.
							for (FPBDRigidParticleHandle* ChildParticle : ClusterUnion->ChildParticles)
							{
								if (ChildParticle)
								{
									if (FPBDRigidClusteredParticleHandle* ClusteredChild = ChildParticle->CastToClustered())
									{
										if (ClusteredChild->ClusterIds().NumChildren > 0)
										{
											ClusteredParticlesToProcess.Add(ClusteredChild);
										}
									}
								}
							}
						}
						else
						{
							// Clustered is inside a clustered union, but not a clustered union itself
							ClusteredParticlesToProcess.Add(Clustered);
						}
					}
					else
					{
						// Clustered is not a clustered union, and not _in_ a clustered union
						ClusteredParticlesToProcess.Add(Clustered);
					}
				}
			}
		}

		BreakingModel(ClusteredParticlesToProcess);
	}
	
	void FRigidClustering::BreakingModel(TArray<FPBDRigidClusteredParticleHandle*>& InParticles)
	{
		BreakingModel(MakeArrayView(InParticles));
	}

	bool FRigidClustering::BreakingModel(TArrayView<FPBDRigidClusteredParticleHandle*> InParticles)
	{
		SCOPE_CYCLE_COUNTER(STAT_BreakingModel);

		FrameProcessedClusters += InParticles.Num();

		MomentumRestoringDataByParticle.Empty();

		bool bHasReleasedParticles = false;
		for(FPBDRigidClusteredParticleHandle* ClusteredParticle : InParticles)
		{
			if(ClusteredParticle->ClusterIds().NumChildren)
			{
				TSet<FPBDRigidParticleHandle*> ActivatedParticles = ReleaseClusterParticles(ClusteredParticle);
				bHasReleasedParticles |= (ActivatedParticles.Num() > 0);
			}
		}

		if (bHasReleasedParticles)
		{
			// This way if we break apart a large cluster union here (i.e. many of its children want to be released from ReleaseClusterParticles due to strain)
			// we'll only update the cluster properties once here (connection graph, geometry, etc.).
			ClusterUnionManager.HandleDeferredClusterUnionUpdateProperties();
			// Restore some of the momentum of objects that were touching rigid clusters that broke
			if (RestoreBreakingMomentumPercent > 0.f)
			{
				RestoreBreakingMomentum();
			}
		}
		MomentumRestoringDataByParticle.Empty();
		return bHasReleasedParticles;
	}

	DECLARE_CYCLE_STAT(TEXT("FRigidClustering::Visitor"), STAT_ClusterVisitor, STATGROUP_Chaos);
	void FRigidClustering::Visitor(FClusterHandle Cluster, FVisitorFunction Function)
	{
		if (Cluster)
		{
			if (MChildren.Contains(Cluster) && MChildren[Cluster].Num())
			{
				SCOPE_CYCLE_COUNTER(STAT_ClusterVisitor);

				// TQueue is a linked list, which has no preallocator.
				TQueue<FRigidHandle> Queue;
				for (Chaos::FPBDRigidParticleHandle* Child : MChildren[Cluster])
				{
					Queue.Enqueue(Child);
				}

				FRigidHandle CurrentHandle = nullptr;
				while (Queue.Dequeue(CurrentHandle))
				{
					if (CurrentHandle)
					{
						if (FClusterHandle CurrentClusterHandle = CurrentHandle->CastToClustered())
						{
							// @question : Maybe we should just store the leaf node bodies in a
							// map, that will require Memory(n*log(n))
							if (MChildren.Contains(CurrentClusterHandle))
							{
								for (Chaos::FPBDRigidParticleHandle* Child : MChildren[CurrentClusterHandle])
								{
									Queue.Enqueue(Child);
								}
							}
						}
						if (CurrentHandle)
						{
							Function(*this, CurrentHandle);
						}
					}
				}
			}
		}
	}

	DECLARE_CYCLE_STAT(TEXT("TPBDRigidClustering<>::GetActiveClusterIndex"), STAT_GetActiveClusterIndex, STATGROUP_Chaos);
	FPBDRigidParticleHandle* 
	FRigidClustering::GetActiveClusterIndex(
		FPBDRigidParticleHandle* Child)
	{
		SCOPE_CYCLE_COUNTER(STAT_GetActiveClusterIndex);
		while (Child && Child->Disabled())
		{
			Child = Child->CastToClustered()->ClusterIds().Id;
		}
		return Child; 
	}

	FPBDRigidParticleHandle* FRigidClustering::FindClosestChild(const FPBDRigidClusteredParticleHandle* ClusteredParticle, const FVec3& WorldLocation) const
	{
		if (const TArray<FPBDRigidParticleHandle*>* ChildrenHandles = GetChildrenMap().Find(ClusteredParticle))
		{
			return FindClosestParticle(*ChildrenHandles, WorldLocation); 
		}
		return nullptr;
	}

	FPBDRigidParticleHandle* FRigidClustering::FindClosestParticle(const TArray<FPBDRigidParticleHandle*>& Particles, const FVec3& WorldLocation)
	{
		FPBDRigidParticleHandle* ClosestChildHandle = nullptr;
		
		// @todo(chaos) we should offer a more precise way to query than the distance from center of mass
		FReal ClosestSquaredDist = TNumericLimits<FReal>::Max();
		for (FPBDRigidParticleHandle* ChildHandle: Particles)
        {
        	const FReal SquaredDist = (ChildHandle->GetX() - WorldLocation).SizeSquared();
        	if (SquaredDist < ClosestSquaredDist)
        	{
        		ClosestSquaredDist = SquaredDist;
        		ClosestChildHandle = ChildHandle;
        	}
        }
        return ClosestChildHandle;
	}

	TArray<FPBDRigidParticleHandle*> FRigidClustering::FindChildrenWithinRadius(const FPBDRigidClusteredParticleHandle* ClusteredParticle, const FVec3& WorldLocation, FReal Radius, bool bAlwaysReturnClosest) const
	{
		if (const TArray<FPBDRigidParticleHandle*>* ChildrenHandles = GetChildrenMap().Find(ClusteredParticle))
		{
			return FindParticlesWithinRadius(*ChildrenHandles, WorldLocation, Radius, bAlwaysReturnClosest); 
		}
		TArray<FPBDRigidParticleHandle*> EmptyArray;
		return EmptyArray;
	}

	TArray<FPBDRigidParticleHandle*> FRigidClustering::FindParticlesWithinRadius(const TArray<FPBDRigidParticleHandle*>& Particles, const FVec3& WorldLocation, FReal Radius, bool bAlwaysReturnClosest)
	{
		TArray<FPBDRigidParticleHandle*> Result;
		
		FPBDRigidParticleHandle* ClosestChildHandle = nullptr;
		
		// @todo(chaos) we should offer a more precise way to query than the distance from center of mass
		FReal ClosestSquaredDist = TNumericLimits<FReal>::Max();
		
		const FReal RadiusSquared = Radius * Radius;
		for (FPBDRigidParticleHandle* ChildHandle: Particles)
		{
			const FReal SquaredDist = (ChildHandle->GetX() - WorldLocation).SizeSquared();
			if (SquaredDist <= RadiusSquared)
			{
				Result.Add(ChildHandle);
			}
			if (bAlwaysReturnClosest)
			{
				if (SquaredDist < ClosestSquaredDist)
				{
					ClosestSquaredDist = SquaredDist;
					ClosestChildHandle = ChildHandle;
				}
			}
		}
		if (bAlwaysReturnClosest && ClosestChildHandle && Result.Num() == 0)
		{
			Result.Add(ClosestChildHandle);
		}
		return Result;
	}
	
	DECLARE_CYCLE_STAT(TEXT("TPBDRigidClustering<>::GenerateConnectionGraph"), STAT_GenerateConnectionGraph, STATGROUP_Chaos);
	void
	FRigidClustering::GenerateConnectionGraph(
		const TArray<FPBDRigidParticleHandle*>& Particles,
		const FClusterCreationParameters& Parameters,
		const TSet<FPBDRigidParticleHandle*>* FromParticles,
		const TSet<FPBDRigidParticleHandle*>* ToParticles)
	{
		SCOPE_CYCLE_COUNTER(STAT_GenerateConnectionGraph);

		// Connectivity Graph
		//    Build a connectivity graph for the cluster. If the PointImplicit is specified
		//    and the ClusterIndex has collision particles then use the expensive connection
		//    method. Otherwise try the DelaunayTriangulation if not none.
		//
		if (Parameters.bGenerateConnectionGraph)
		{
			FClusterCreationParameters::EConnectionMethod LocalConnectionMethod = Parameters.ConnectionMethod;

			if (LocalConnectionMethod == FClusterCreationParameters::EConnectionMethod::None)
			{
				LocalConnectionMethod = FClusterCreationParameters::EConnectionMethod::MinimalSpanningSubsetDelaunayTriangulation; // default method
			}

			if (LocalConnectionMethod == FClusterCreationParameters::EConnectionMethod::PointImplicit ||
				LocalConnectionMethod == FClusterCreationParameters::EConnectionMethod::PointImplicitAugmentedWithMinimalDelaunay)
			{
				UpdateConnectivityGraphUsingPointImplicit(Particles, Parameters.CoillisionThicknessPercent, FromParticles, ToParticles);
			}

			if (LocalConnectionMethod == FClusterCreationParameters::EConnectionMethod::DelaunayTriangulation)
			{
				UpdateConnectivityGraphUsingDelaunayTriangulation(Particles, Parameters, FromParticles, ToParticles); // not thread safe
			}

			if (LocalConnectionMethod == FClusterCreationParameters::EConnectionMethod::BoundsOverlapFilteredDelaunayTriangulation)
			{
				UpdateConnectivityGraphUsingDelaunayTriangulationWithBoundsOverlaps(Particles, Parameters, FromParticles, ToParticles);
			}

			if (LocalConnectionMethod == FClusterCreationParameters::EConnectionMethod::PointImplicitAugmentedWithMinimalDelaunay ||
				LocalConnectionMethod == FClusterCreationParameters::EConnectionMethod::MinimalSpanningSubsetDelaunayTriangulation)
			{
				FixConnectivityGraphUsingDelaunayTriangulation(Particles, Parameters, FromParticles, ToParticles);
			}
		}
	}

	void 
	FRigidClustering::GenerateConnectionGraph(
		Chaos::FPBDRigidClusteredParticleHandle* Parent,
		const FClusterCreationParameters& Parameters)
	{
		if (!MChildren.Contains(Parent))
			return;

		FClusterCreationParameters FinalParameters = Parameters;
		if (Parameters.ConnectionMethod == FClusterCreationParameters::EConnectionMethod::PointImplicit && !Parent->CollisionParticles())
		{
			FinalParameters.ConnectionMethod = FClusterCreationParameters::EConnectionMethod::MinimalSpanningSubsetDelaunayTriangulation;
		}

		GenerateConnectionGraph(MChildren[Parent], FinalParameters);
	}

	DECLARE_CYCLE_STAT(TEXT("TPBDRigidClustering<>::ClearConnectionGraph"), STAT_ClearConnectionGraph, STATGROUP_Chaos);
	void
	FRigidClustering::ClearConnectionGraph(FPBDRigidClusteredParticleHandle* Parent)
	{
		SCOPE_CYCLE_COUNTER(STAT_ClearConnectionGraph);
		if (!MChildren.Contains(Parent))
		{
			return;
		}

		const TArray<FPBDRigidParticleHandle*>& Children = MChildren[Parent];
		for (FPBDRigidParticleHandle* Handle : Children)
		{
			if (!Handle)
			{
				continue;
			}

			RemoveNodeConnections(Handle);
		}
	}

	FRealSingle MinImpulseForStrainEval = 980 * 2 * 1.f / 30.f; //ignore impulses caused by just keeping object on ground. This is a total hack, we should not use accumulated impulse directly. Instead we need to look at delta v along constraint normal
	FAutoConsoleVariableRef CVarMinImpulseForStrainEval(TEXT("p.chaos.MinImpulseForStrainEval"), MinImpulseForStrainEval, TEXT("Minimum accumulated impulse before accumulating for strain eval "));

	bool bUseContactSpeedForStrainThreshold = true;
	FAutoConsoleVariableRef CVarUseContactSpeedForStrainEval(TEXT("p.chaos.UseContactSpeedForStrainEval"), bUseContactSpeedForStrainThreshold, TEXT("Whether to use contact speed to discard contacts when updating cluster strain (true: use speed, false: use impulse)"));

	FRealSingle MinContactSpeedForStrainEval = 1.0f; // Ignore contacts where the two bodies are resting together
	FAutoConsoleVariableRef CVarMinContactSpeedForStrainEval(TEXT("p.chaos.MinContactSpeedForStrainEval"), MinContactSpeedForStrainEval, TEXT("Minimum speed at the contact before accumulating for strain eval "));

	DECLARE_CYCLE_STAT(TEXT("ComputeStrainFromCollision"), STAT_ComputeStrainFromCollision, STATGROUP_Chaos);
	void FRigidClustering::ComputeStrainFromCollision(const FPBDCollisionConstraints& CollisionRule, const FReal Dt)
	{
		SCOPE_CYCLE_COUNTER(STAT_ComputeStrainFromCollision);
		const FRealSingle InvDt = (Dt > SMALL_NUMBER) ? (FRealSingle)(1.0 / Dt) : 1.0f;

		ResetCollisionImpulseArray();

		for (Chaos::FPBDCollisionConstraintHandle* ContactHandle : CollisionRule.GetConstraintHandles())
		{
			if (ContactHandle == nullptr)
			{
				continue;
			}

			if (ContactHandle->GetContact().GetIsOneWayInteraction())
			{
				continue;
			}

			TVector<FGeometryParticleHandle*, 2> ConstrainedParticles = ContactHandle->GetConstrainedParticles();
			
			// make sure we only compute things if one of the two particle is clustered
			FPBDRigidClusteredParticleHandle* ClusteredConstrainedParticles0 = ConstrainedParticles[0]->CastToClustered();
			FPBDRigidClusteredParticleHandle* ClusteredConstrainedParticles1 = ConstrainedParticles[1]->CastToClustered();
			if (!ClusteredConstrainedParticles0 && !ClusteredConstrainedParticles1)
			{
				continue;
			}

			const FPBDRigidParticleHandle* Rigid0 = ConstrainedParticles[0]->CastToRigidParticle();
			const FPBDRigidParticleHandle* Rigid1 = ConstrainedParticles[1]->CastToRigidParticle();

			if (bUseContactSpeedForStrainThreshold)
			{
				// Get dV between the two particles and project onto the normal to get the approach speed (take PreV as V is the new velocity post-solve)
				const FVec3 V0 = Rigid0 ? Rigid0->GetPreV() : FVec3(0);
				const FVec3 V1 = Rigid1 ? Rigid1->GetPreV() : FVec3(0);
				const FVec3 DeltaV = V0 - V1;
				const FReal SpeedAlongNormal = FVec3::DotProduct(DeltaV, ContactHandle->GetContact().CalculateWorldContactNormal());

				// If we're not approaching at more than the min speed, reject the contact
				if (SpeedAlongNormal > -MinContactSpeedForStrainEval && ContactHandle->GetAccumulatedImpulse().SizeSquared() > FReal(0))
				{
					continue;
				}
			}
			else if (ContactHandle->GetAccumulatedImpulse().Size() < MinImpulseForStrainEval)
			{
				continue;
			}

			auto ComputeStrainLambda = [this, &ContactHandle, &InvDt](
				FPBDRigidClusteredParticleHandle* Cluster,
				FRealSingle& OutTotalImpulseAccumulator)
			{
				const FVec3 ContactWorldLocation = ContactHandle->GetContact().CalculateWorldContactLocation();
				
				const FRealSingle AccumulatedImpulse = static_cast<FRealSingle>(ContactHandle->GetAccumulatedImpulse().Size());
				if (AccumulatedImpulse > UE_SMALL_NUMBER && FMath::IsFinite(AccumulatedImpulse))
				{
					if (const FClusterUnionPhysicsProxy* ClusterUnionProxy = GetConcreteProxy<FClusterUnionPhysicsProxy>(Cluster))
					{
						// check if this cluster union accept damage from collision 
						if (!ClusterUnionProxy->GetEnableStrainOnCollision_Internal())
						{
							return;
						}

						FPBDRigidParticleHandle* ChildParticleInContact = nullptr;
						if (Chaos::FClusterUnion* ClusterUnion = ClusterUnionManager.FindClusterUnionFromParticle(Cluster))
						{
							// At the moment, we don't want to apply strains to children of ClusterUnions, we want instead
							// to apply the strains to GRANDchildren of ClusterUnions.
							// We are looking for the child particle that match the shape in contact 
							// We cannot use a proximity method because this may return the wrong child if its center of mass is closer than actual child in contact
							// Like in the example below where A would be wrongly reported:
							//               +-------+
							//               |   x A |
							//            +--+-------+--------------------------------+
							// contact-> *|                    x B                    |
							//            +-------------------------------------------+
							int32 ShapeInContactIndex = ContactHandle->GetContact().GetShape0()->GetShapeIndex();
							if (Cluster == ContactHandle->GetContact().GetParticle1())
							{
								ShapeInContactIndex = ContactHandle->GetContact().GetShape1()->GetShapeIndex();
							}

							// cluster union garantee a one to one mapping between shapes index and children
							// ( note : we cannot use MChildren because it can be oput of order, so we need to use the ChildParticles from FClusterUnion ) 
							if (ensureAlways(ClusterUnion->ChildParticles.IsValidIndex(ShapeInContactIndex)))
							{
								ChildParticleInContact = ClusterUnion->ChildParticles[ShapeInContactIndex];
							}

							if (ChildParticleInContact == nullptr)
							{
								// if anything above failed to find a particle fall back to the distance based method
								ChildParticleInContact = FindClosestChild(Cluster, ContactWorldLocation);
							}
						}
						// If Closest child is not a clustered, then there is no substructure to apply strain to,
						// so null Cluster
						Cluster
							= ChildParticleInContact
							? ChildParticleInContact->CastToClustered()
							: nullptr;
					}

					if (Cluster == nullptr)
					{
						return;
					}

					// gather propagation information from the parent proxy
					bool bUseDamagePropagation = false;
					EDamageEvaluationModel DamageModel = EDamageEvaluationModel::StrainFromDamageThreshold;
					if (const FGeometryCollectionPhysicsProxy* GeometryCollectionProxy = GetConcreteProxy<FGeometryCollectionPhysicsProxy>(Cluster))
					{
						const FSimulationParameters& SimParams = GeometryCollectionProxy->GetSimParameters();
						bUseDamagePropagation = SimParams.bUseDamagePropagation;
						DamageModel = SimParams.DamageEvaluationModel;
						if (!SimParams.bEnableStrainOnCollision)
						{
							return;
						}
					}

					FRealSingle CollisionImpulseToApply = AccumulatedImpulse;
					if (DamageModel == EDamageEvaluationModel::StrainFromMaterialStrengthAndConnectivity)
					{
						// this damage model use internal strain as the maximum breaking force (m.a), but we need to compare to impulses (m.v), let's divide this impulse by Dt
						// technically this is no longer an impulse but we intend in the future to get rid of COllisionImpulse property and use external strain instead
						CollisionImpulseToApply *= InvDt;
					}

					if (Cluster->ClusterIds().NumChildren == 0)
					{
						// Special case that can happen when we're looking to hit a cluster union with partial destruction as a result of intercluster edges.
						// We might hit a leaf particle in a geometry collection which has 0 children. In that case, we actually just want to apply strain to the
						// particle itself rather than trying to find its children.
						Cluster->CollisionImpulses() += CollisionImpulseToApply;
						UpdateTopLevelParticle(Cluster);
						OutTotalImpulseAccumulator += CollisionImpulseToApply;
					}
					else if (bUseDamagePropagation)
					{
						// propagation based breaking model start from the closest particle and propagate through the connection graph
						// propagation logic is dealt when evaluating the strain
						if (FPBDRigidParticleHandle* ClosestChild = FindClosestChild(Cluster, ContactWorldLocation))
						{
							if (TPBDRigidClusteredParticleHandle<FReal, 3>* ClusteredChild = ClosestChild->CastToClustered())
							{
								ClusteredChild->CollisionImpulses() += CollisionImpulseToApply;
								UpdateTopLevelParticle(ClusteredChild);
								OutTotalImpulseAccumulator += CollisionImpulseToApply;
							}
						}
					}
					else
					{
						const FRigidTransform3 WorldToClusterTM = FRigidTransform3(Cluster->GetP(), Cluster->GetQ());
						const FVec3 ContactLocationClusterLocal = WorldToClusterTM.InverseTransformPosition(ContactWorldLocation);
						FAABB3 ContactBox(ContactLocationClusterLocal, ContactLocationClusterLocal);
						ContactBox.Thicken(ClusterDistanceThreshold);
						if (Cluster->GetChildrenSpatial())
						{
							// todo(chaos): FindAllIntersectingChildren may return an unfiltered list of children ( when num children is under a certain threshold )   
							const TArray<FPBDRigidParticleHandle*> Intersections = Cluster->GetChildrenSpatial()->FindAllIntersectingChildren(ContactBox);
							for (FPBDRigidParticleHandle* Child : Intersections)
							{
								if (TPBDRigidClusteredParticleHandle<FReal, 3>*ClusteredChild = Child->CastToClustered())
								{
									ClusteredChild->CollisionImpulses() += CollisionImpulseToApply;
									UpdateTopLevelParticle(ClusteredChild);
									OutTotalImpulseAccumulator += CollisionImpulseToApply;
								}
							}
						}
					}
				}
			};

			// We only need to dirty the impulse array if any of the active contacts actually added 
			// a collision impulse to a particle. If they are all resting or otherwise non-impulsive
			// contacts then we can skip dirtying the impulse array and avoid running the breaking
			// model when we know nothing will break
			FRealSingle TotalImpulses[] = { 0.0f, 0.0f };

			if (ClusteredConstrainedParticles0)
			{
				ComputeStrainLambda(ClusteredConstrainedParticles0, TotalImpulses[0]);
				MCollisionImpulseArrayDirty |= TotalImpulses[0] > 0.0f;
			}

			if (ClusteredConstrainedParticles1)
			{
				ComputeStrainLambda(ClusteredConstrainedParticles1, TotalImpulses[1]);
				MCollisionImpulseArrayDirty |= TotalImpulses[1] > 0.0f;
			}
		}
	}

	DECLARE_CYCLE_STAT(TEXT("ResetCollisionImpulseArray"), STAT_ResetCollisionImpulseArray, STATGROUP_Chaos);
	void FRigidClustering::ResetCollisionImpulseArray()
	{
		SCOPE_CYCLE_COUNTER(STAT_ResetCollisionImpulseArray);
		if (MCollisionImpulseArrayDirty)
		{
			FPBDRigidsSOAs& ParticleStructures = MEvolution.GetParticles();
			ParticleStructures.GetGeometryCollectionParticles().CollisionImpulsesArray().Fill(0.0f);
			ParticleStructures.GetClusteredParticles().CollisionImpulsesArray().Fill(0.0f);
			MCollisionImpulseArrayDirty = false;
		}
	}

	void FRigidClustering::DisableCluster(FPBDRigidClusteredParticleHandle* ClusteredParticle)
	{
		// #note: we don't recursively descend to the children
		DisableParticle(ClusteredParticle);
		ClusteredParticle->ClusterIds() = ClusterId();
		ClusteredParticle->ClusterGroupIndex() = 0;
	}

	void FRigidClustering::DisableParticle(FPBDRigidParticleHandle* ParticleToDisable, bool bRemoveFromChildrenMap)
	{
		if (ParticleToDisable)
		{
			// Before disabling the particle, we need to make sure to remove the recorded breaking collision from BreakingCollisions
			// to avoid it crashing later when processing it 
			RemoveFromMomentumRestoringStructures(ParticleToDisable);

			MEvolution.DisableParticle(ParticleToDisable);
			if (FPBDRigidClusteredParticleHandle* ClusteredParticle = ParticleToDisable->CastToClustered())
			{
				TopLevelClusterParents.Remove(ClusteredParticle);
				TopLevelClusterParentsStrained.Remove(ClusteredParticle);
				if (bRemoveFromChildrenMap)
				{
					MChildren.Remove(ClusteredParticle);
				}
			}
		}
	}

	void FRigidClustering::ApplyStrainModifiers(const TArray<FPBDRigidClusteredParticleHandle*>& StrainedParticles)
	{
		if (StrainModifiers)
		{
			for (ISimCallbackObject* Modifier : *StrainModifiers)
			{
				FStrainModifierAccessor Accessor(*this, &StrainedParticles);
				Modifier->StrainModification_Internal(Accessor);
			}
		}
	}

	FPBDRigidClusteredParticleHandle*
	FRigidClustering::DestroyClusterParticle(
		FPBDRigidClusteredParticleHandle* ClusteredParticle,
		const FClusterDestoryParameters& Parameters)
	{
		FClusterHandle ParentParticle = nullptr;

		// Before derstroying the particle, we need to make sure to remove the recorded breaking collision from BreakingCollisions
		// to avoid it crashing later when processing it 
		RemoveFromMomentumRestoringStructures(ClusteredParticle);

		// detach connections to thie parent from the children
		if (MChildren.Contains(ClusteredParticle))
		{
			for (FRigidHandle Child : MChildren[ClusteredParticle])
			{
				if (FClusterHandle ClusteredChild = Child->CastToClustered())
				{
					ClusteredChild->ClusterIds() = ClusterId();
					ClusteredChild->ClusterGroupIndex() = 0;
				}
			}

			MChildren.Remove(ClusteredParticle);
		}

		// disable within the solver
		if (!ClusteredParticle->Disabled())
		{
			DisableParticle(ClusteredParticle);
			ensure(ClusteredParticle->ClusterIds().Id == nullptr);
		}


		// need to disconnect from any other particles ( this can be from being a child of a cluster or a cluster union)
		RemoveNodeConnections(ClusteredParticle);

		// disconnect from the parents
		if (ClusteredParticle->ClusterIds().Id)
		{
			ParentParticle = ClusteredParticle->Parent();

			// Need to also check if the particle is a cluster union and remove from that as well.
			// This needs to be before the call to clear our ClusterIds since HandleRemoveOperationWithClusterLookup needs to use the particle's parent to find the right cluster union.
			ClusterUnionManager.HandleRemoveOperationWithClusterLookup({ ClusteredParticle }, EClusterUnionOperationTiming::Defer);

			ClusteredParticle->ClusterIds() = ClusterId();
			ClusteredParticle->ClusterGroupIndex() = 0;

			if (MChildren.Contains(ParentParticle))
			{
				FRigidHandleArray& Children = MChildren[ParentParticle];

				// disconnect from your parents children list
				Children.Remove(ClusteredParticle);

				// disable internal parents that have lost all their children - do not try to disable cluster unions.
				if (!MChildren[ParentParticle].Num() && ParentParticle->InternalCluster() && !ClusterUnionManager.IsClusterUnionParticle(ParentParticle))
				{
					DisableCluster(ClusteredParticle);
				}
			}
		}

		// remove internal parents that have no children. 
		if (ClusteredParticle->InternalCluster())
		{
			FUniqueIdx UniqueIdx = ClusteredParticle->UniqueIdx();
			MEvolution.DestroyParticle(ClusteredParticle);
			MEvolution.ReleaseUniqueIdx(UniqueIdx);
		}

		if (Parameters.bReturnInternalOnly && ParentParticle && !ParentParticle->InternalCluster())
		{
			ParentParticle = nullptr;
		}

		// reset the structures
		// Note: this needs to be at the end to make sure that no other operations above may re-add it 
		// ( for example HandleRemoveOperationWithClusterLookup )
		TopLevelClusterParents.Remove(ClusteredParticle);
		TopLevelClusterParentsStrained.Remove(ClusteredParticle);

		return ParentParticle;

	}

	bool FRigidClustering::BreakCluster(FPBDRigidClusteredParticleHandle* ClusteredParticle)
	{
		if (!ClusteredParticle)
		{
			return false;
		}

		// max strain will allow to unconditionally release the children when strain is evaluated
		constexpr FRealSingle MaxStrain = TNumericLimits<FRealSingle>::Max();
		if (const TArray<FPBDRigidParticleHandle*>* ChildrenHandles = GetChildrenMap().Find(ClusteredParticle))
		{
			for (FPBDRigidParticleHandle* ChildHandle: *ChildrenHandles)
			{
				if (Chaos::FPBDRigidClusteredParticleHandle* ClusteredChildHandle = ChildHandle->CastToClustered())
				{
					ClusteredChildHandle->SetExternalStrains(MaxStrain);
					SetExternalStrain(ClusteredChildHandle, MaxStrain);
					// This way we won't try to propagate this infinite strain.
					ClusteredChildHandle->SetInternalStrains(ClusteredChildHandle->GetExternalStrain());
				}
			}
			if (ChildrenHandles->Num() > 0)
			{
				CrumbledSinceLastUpdate.Add(ClusteredParticle);
				SendCrumblingEvent(ClusteredParticle);
			}
			return true;
		}
		return false;
	}

	bool FRigidClustering::BreakClustersByProxy(const IPhysicsProxyBase* Proxy)
	{
		bool bCrumbledAnyCluster = false;
		// max strain will allow to unconditionally release the children when strain is evaluated
		constexpr FRealSingle MaxStrain = TNumericLimits<FRealSingle>::Max();

		// we should probably have a way to retrieve all the active clusters per proxy instead of having to do this iteration
		for (FPBDRigidClusteredParticleHandle* ClusteredHandle : GetTopLevelClusterParents())
		{
			if (!ClusteredHandle)
			{
				continue;
			}

			const bool bIsInputProxy = ClusteredHandle->PhysicsProxy() == Proxy;

			// This handles the case where we want to break a GC but it's still in a cluster union.
			const bool bIsInPhysicsProxiesSet = ClusteredHandle->PhysicsProxies().Contains(Proxy);
			if (bIsInputProxy || bIsInPhysicsProxiesSet)
			{
				// Now we need to go from the parent cluster union particle to the GC particle that corresponds to the proxy.
				if (bIsInPhysicsProxiesSet && !bIsInputProxy)
				{
					if (TArray<FPBDRigidParticleHandle*>* Children = MChildren.Find(ClusteredHandle))
					{
						ClusteredHandle = nullptr;
						FPBDRigidParticleHandle** Candidate = Children->FindByPredicate(
							[Proxy](FPBDRigidParticleHandle* Particle)
							{
								return Particle->PhysicsProxy() == Proxy && Particle->CastToClustered();
							}
						);

						if (Candidate)
						{
							ClusteredHandle = (*Candidate)->CastToClustered();
						}
					}

					if (!ClusteredHandle)
					{
						continue;
					}
				}

				if (const TArray<FPBDRigidParticleHandle*>* Children = MChildren.Find(ClusteredHandle))
				{
					for (FPBDRigidParticleHandle* ChildHandle : *Children)
					{
						if (Chaos::FPBDRigidClusteredParticleHandle* ClusteredChildHandle = ChildHandle->CastToClustered())
						{
							SetExternalStrain(ClusteredChildHandle, MaxStrain);
							// This way we won't try to propagate this infinite strain.
							ClusteredChildHandle->SetInternalStrains(ClusteredChildHandle->GetExternalStrain());
						}
					}
					if (Children->Num() > 0)
					{
						CrumbledSinceLastUpdate.Add(ClusteredHandle);
						SendCrumblingEvent(ClusteredHandle);
					}
				}
				bCrumbledAnyCluster = true;
			}
		}

		return bCrumbledAnyCluster;
	}

	static void ConnectClusteredNodes(FPBDRigidClusteredParticleHandle* ClusteredChild1, FPBDRigidClusteredParticleHandle* ClusteredChild2, const TSet<FPBDRigidParticleHandle*>* FromParticles, const TSet<FPBDRigidParticleHandle*>* ToParticles)
	{
		check(ClusteredChild1 && ClusteredChild2);
		if (ClusteredChild1 == ClusteredChild2)
		{
			return;
		}

		if (FromParticles && ToParticles)
		{
			// We are enforcing that the edge we create crosses from the "from" set to the "to" set (or vice versa).
			const bool bInFrom = FromParticles->Contains(ClusteredChild1) || FromParticles->Contains(ClusteredChild2);
			const bool bInTo = ToParticles->Contains(ClusteredChild1) || ToParticles->Contains(ClusteredChild2);
			if (!bInFrom || !bInTo)
			{
				return;
			}
		}


		const FRealSingle AvgStrain = (ClusteredChild1->GetInternalStrains() + ClusteredChild2->GetInternalStrains()) * 0.5f;
		TArray<TConnectivityEdge<FReal>>& Edges1 = ClusteredChild1->ConnectivityEdges();
		TArray<TConnectivityEdge<FReal>>& Edges2 = ClusteredChild2->ConnectivityEdges();
		if (//Edges1.Num() < Parameters.MaxNumConnections && 
			!Edges1.FindByKey(ClusteredChild2))
		{
			Edges1.Add(TConnectivityEdge<FReal>(ClusteredChild2, AvgStrain));
		}
		if (//Edges2.Num() < Parameters.MaxNumConnections && 
			!Edges2.FindByKey(ClusteredChild1))
		{
			Edges2.Add(TConnectivityEdge<FReal>(ClusteredChild1, AvgStrain));
		}
	}
	
	static void ConnectNodes(FPBDRigidParticleHandle* Child1, FPBDRigidParticleHandle* Child2, const TSet<FPBDRigidParticleHandle*>* FromParticles, const TSet<FPBDRigidParticleHandle*>* ToParticles)
	{
		check(Child1 != Child2);
		FPBDRigidClusteredParticleHandle* ClusteredChild1 = Child1->CastToClustered();
		FPBDRigidClusteredParticleHandle* ClusteredChild2 = Child2->CastToClustered();
		ConnectClusteredNodes(ClusteredChild1, ClusteredChild2, FromParticles, ToParticles);
	}

	void FRigidClustering::CreateNodeConnection(FPBDRigidClusteredParticleHandle* A, FPBDRigidClusteredParticleHandle* B)
	{
		ConnectClusteredNodes(A, B, nullptr, nullptr);
	}

	DECLARE_CYCLE_STAT(TEXT("TPBDRigidClustering<>::UpdateConnectivityGraphUsingPointImplicit"), STAT_UpdateConnectivityGraphUsingPointImplicit, STATGROUP_Chaos);
	void 
	FRigidClustering::UpdateConnectivityGraphUsingPointImplicit(
		const TArray<FPBDRigidParticleHandle*>& Particles,
		FReal CollisionThicknessPercent,
		const TSet<FPBDRigidParticleHandle*>* FromParticles,
		const TSet<FPBDRigidParticleHandle*>* ToParticles)
	{
		SCOPE_CYCLE_COUNTER(STAT_UpdateConnectivityGraphUsingPointImplicit);

		if (!UseConnectivity)
		{
			return;
		}

		const FReal Delta = FMath::Min(FMath::Max(CollisionThicknessPercent, FReal(0)), FReal(1));

		typedef TPair<FPBDRigidParticleHandle*, FPBDRigidParticleHandle*> ParticlePair;
		typedef TSet<ParticlePair> ParticlePairArray;

		TArray<ParticlePairArray> Connections;
		Connections.Init(ParticlePairArray(), Particles.Num());

		PhysicsParallelFor(Particles.Num(), [&](int32 i)
			{
				FPBDRigidParticleHandle* Child1 = Particles[i];
				if (Child1->GetGeometry() && Child1->GetGeometry()->HasBoundingBox())
				{
					ParticlePairArray& ConnectionList = Connections[i];

					const FVec3& Child1X = Child1->GetX();
					FRigidTransform3 TM1 = FRigidTransform3(Child1X, Child1->GetR());

					const int32 Offset = i + 1;
					const int32 NumRemainingParticles = Particles.Num() - Offset;

					for (int32 Idx = 0; Idx < NumRemainingParticles; ++Idx)
					{
						const int32 ParticlesIdx = Offset + Idx;
						FPBDRigidParticleHandle* Child2 = Particles[ParticlesIdx];
						if (Child2->CollisionParticles())
						{

							const FVec3& Child2X = Child2->GetX();
							const FRigidTransform3 TM = TM1.GetRelativeTransform(FRigidTransform3(Child2X, Child2->GetR()));
							const uint32 NumCollisionParticles = Child2->CollisionParticles()->Size();
							for (uint32 CollisionIdx = 0; CollisionIdx < NumCollisionParticles; ++CollisionIdx)
							{
								const FVec3 LocalPoint =
									TM.TransformPositionNoScale(Child2->CollisionParticles()->GetX(CollisionIdx));
								const FReal Phi = Child1->GetGeometry()->SignedDistance(LocalPoint - (LocalPoint * Delta));
								if (Phi < 0.0)
								{
									ConnectionList.Add(ParticlePair(Child1, Child2));
									break;
								}

							}
						}
					}
				}
			});

		// join results and make connections
		for (const ParticlePairArray& ConnectionList : Connections)
		{
			for (const ParticlePair& Edge : ConnectionList)
			{
				ConnectNodes(Edge.Key, Edge.Value, FromParticles, ToParticles);
			}
		}

	}

	void
	FRigidClustering::UpdateConnectivityGraphUsingPointImplicit(
		Chaos::FPBDRigidClusteredParticleHandle* Parent,
		const FClusterCreationParameters& Parameters)
	{
		const TArray<FPBDRigidParticleHandle*>& Children = MChildren[Parent];
		UpdateConnectivityGraphUsingPointImplicit(Children, Parameters.CoillisionThicknessPercent);
	}

	DECLARE_CYCLE_STAT(TEXT("TPBDRigidClustering<>::FixConnectivityGraphUsingDelaunayTriangulation"), STAT_FixConnectivityGraphUsingDelaunayTriangulation, STATGROUP_Chaos);
	void
	FRigidClustering::FixConnectivityGraphUsingDelaunayTriangulation(
		const TArray<FPBDRigidParticleHandle*>& Particles,
		const FClusterCreationParameters& Parameters,
		const TSet<FPBDRigidParticleHandle*>* FromParticles,
		const TSet<FPBDRigidParticleHandle*>* ToParticles)
	{
		SCOPE_CYCLE_COUNTER(STAT_FixConnectivityGraphUsingDelaunayTriangulation);

		// Compute Delaunay neighbor graph on Particles centers
		TArray<FVector> Pts;
		Pts.AddUninitialized(Particles.Num());
		for (int32 i = 0; i < Particles.Num(); ++i)
		{
			Pts[i] = Particles[i]->GetX();
		}
		TArray<TArray<int32>> Neighbors; // Indexes into Particles
		VoronoiNeighbors(Pts, Neighbors);

		// Build a UnionFind graph to find (indirectly) connected Particles
		struct UnionFindInfo
		{
			FPBDRigidParticleHandle* GroupId;
			int32 Size;
		};
		TMap<FPBDRigidParticleHandle*, UnionFindInfo> UnionInfo;
		UnionInfo.Reserve(Particles.Num());

		// Initialize UnionInfo:
		//		0: GroupId = Particles[0], Size = 1
		//		1: GroupId = Particles[1], Size = 1
		//		2: GroupId = Particles[2], Size = 1
		//		3: GroupId = Particles[3], Size = 1

		for (FPBDRigidParticleHandle* Child : Particles)
		{
			UnionInfo.Add(Child, { Child, 1 }); // GroupId, Size
		}

		auto FindGroup = [&](FPBDRigidParticleHandle* Id)
		{
			FPBDRigidParticleHandle* GroupId = Id;
			if (GroupId)
			{
				int findIters = 0;
				while (UnionInfo[GroupId].GroupId != GroupId)
				{
					ensure(findIters++ < 10); // if this while loop iterates more than a few times, there's probably a bug in the unionfind
					auto& CurrInfo = UnionInfo[GroupId];
					auto& NextInfo = UnionInfo[CurrInfo.GroupId];
					CurrInfo.GroupId = NextInfo.GroupId;
					GroupId = NextInfo.GroupId;
					if (!GroupId) break; // error condidtion
				}
			}
			return GroupId;
		};

		// MergeGroup(Particles[0], Particles[1])
		//		0: GroupId = Particles[1], Size = 0
		//		1: GroupId = Particles[1], Size = 2
		//		2: GroupId = Particles[2], Size = 1
		//		3: GroupId = Particles[3], Size = 1

		auto MergeGroup = [&](FPBDRigidParticleHandle* A, FPBDRigidParticleHandle* B)
		{
			FPBDRigidParticleHandle* GroupA = FindGroup(A);
			FPBDRigidParticleHandle* GroupB = FindGroup(B);
			if (GroupA == GroupB)
			{
				return;
			}
			// Make GroupA the smaller of the two
			if (UnionInfo[GroupA].Size > UnionInfo[GroupB].Size)
			{
				Swap(GroupA, GroupB);
			}
			// Overwrite GroupA with GroupB
			UnionInfo[GroupA].GroupId = GroupB;
			UnionInfo[GroupB].Size += UnionInfo[GroupA].Size;
			UnionInfo[GroupA].Size = 0; // not strictly necessary, but more correct
		};

		// Merge all groups with edges connecting them.
		for (int32 i = 0; i < Particles.Num(); ++i)
		{
			FPBDRigidParticleHandle* Child = Particles[i];
			const TArray<TConnectivityEdge<FReal>>& Edges = Child->CastToClustered()->ConnectivityEdges();
			for (const TConnectivityEdge<FReal>& Edge : Edges)
			{
				if (UnionInfo.Contains(Edge.Sibling))
				{
					MergeGroup(Child, Edge.Sibling);
				}
			}
		}

		// Find candidate edges from the Delaunay graph to consider adding
		struct LinkCandidate
		{
			//int32 A, B;
			FPBDRigidParticleHandle* A;
			FPBDRigidParticleHandle* B;
			FReal DistSq;
		};
		TArray<LinkCandidate> Candidates;
		Candidates.Reserve(Neighbors.Num());

		const FReal AlwaysAcceptBelowDistSqThreshold = 50.f * 50.f * 100.f * MClusterConnectionFactor;
		for (int32 i = 0; i < Neighbors.Num(); i++)
		{
			FPBDRigidParticleHandle* Child1 = Particles[i];
			const TArray<int32>& Child1Neighbors = Neighbors[i];
			for (const int32 Nbr : Child1Neighbors)
			{
				if (Nbr < i)
				{ // assume we'll get the symmetric connection; don't bother considering this one
					continue;
				}
				FPBDRigidParticleHandle* Child2 = Particles[Nbr];

				const FReal DistSq = FVector::DistSquared(Pts[i], Pts[Nbr]);
				if (DistSq < AlwaysAcceptBelowDistSqThreshold)
				{ // below always-accept threshold: don't bother adding to candidates array, just merge now
					MergeGroup(Child1, Child2);
					ConnectNodes(Child1, Child2, FromParticles, ToParticles);
					continue;
				}

				if (FindGroup(Child1) == FindGroup(Child2))
				{ // already part of the same group so we don't need Delaunay edge  
					continue;
				}

				// add to array to sort and add as-needed
				Candidates.Add({ Child1, Child2, DistSq });
			}
		}

		// Only add edges that would connect disconnected components, considering shortest edges first
		Candidates.Sort([](const LinkCandidate& A, const LinkCandidate& B) { return A.DistSq < B.DistSq; });
		for (const LinkCandidate& Candidate : Candidates)
		{
			FPBDRigidParticleHandle* Child1 = Candidate.A;
			FPBDRigidParticleHandle* Child2 = Candidate.B;
			if (FindGroup(Child1) != FindGroup(Child2))
			{
				MergeGroup(Child1, Child2);
				ConnectNodes(Child1, Child2, FromParticles, ToParticles);
			}
		}
	}

	void 
	FRigidClustering::FixConnectivityGraphUsingDelaunayTriangulation(
		Chaos::FPBDRigidClusteredParticleHandle* Parent,
		const FClusterCreationParameters& Parameters)
	{
		const TArray<FPBDRigidParticleHandle*>& Children = MChildren[Parent];
		FixConnectivityGraphUsingDelaunayTriangulation(Children, Parameters);
	}
	
	// connection filters
	static bool IsAlwaysValidConnection(const FPBDRigidParticleHandle* Child1, const FPBDRigidParticleHandle* Child2)
	{
		return true;
	}
	
	static bool IsOverlappingConnection(const FPBDRigidParticleHandle* Child1, const FPBDRigidParticleHandle* Child2, FReal Margin)
	{
		if (ensure(Child1 != nullptr && Child2 != nullptr ))
		{
			FAABB3 Bounds1 = Child1->WorldSpaceInflatedBounds();
			Bounds1.Thicken(Margin);
			return Bounds1.Intersects(Child2->WorldSpaceInflatedBounds());
		}
		return false;
	}

	template <typename FilterLambda>
	static void UpdateConnectivityGraphUsingDelaunayTriangulationWithFiltering(
		const TArray<FPBDRigidParticleHandle*>& Children,
		FilterLambda ShouldKeepConnection,
		const TSet<FPBDRigidParticleHandle*>* FromParticles,
		const TSet<FPBDRigidParticleHandle*>* ToParticles)
	{
		TArray<FVector> Pts;
		Pts.AddUninitialized(Children.Num());
		for (int32 i = 0; i < Children.Num(); ++i)
		{
			Pts[i] = Children[i]->GetX();
		}
		TArray<TArray<int>> Neighbors;
		VoronoiNeighbors(Pts, Neighbors);

		TSet<TPair<FPBDRigidParticleHandle*, FPBDRigidParticleHandle*>> UniqueEdges;
		for (int32 i = 0; i < Neighbors.Num(); i++)
		{
			for (int32 j = 0; j < Neighbors[i].Num(); j++)
			{
				FPBDRigidParticleHandle* Child1 = Children[i];
				FPBDRigidParticleHandle* Child2 = Children[Neighbors[i][j]];
				const bool bFirstSmaller = Child1 < Child2;
				TPair<FPBDRigidParticleHandle*, FPBDRigidParticleHandle*> SortedPair(
					bFirstSmaller ? Child1 : Child2, 
					bFirstSmaller ? Child2 : Child1);
				if (!UniqueEdges.Find(SortedPair))
				{
					if (ShouldKeepConnection(Child1, Child2))
					{
						// this does not use ConnectNodes because Neighbors is bi-direction : as in (1,2),(2,1)
						ConnectNodes(Child1, Child2, FromParticles, ToParticles);
						UniqueEdges.Add(SortedPair);
					}
				}
			}
		}
	}
	
	DECLARE_CYCLE_STAT(TEXT("TPBDRigidClustering<>::UpdateConnectivityGraphUsingDelaunayTriangulation"), STAT_UpdateConnectivityGraphUsingDelaunayTriangulation, STATGROUP_Chaos);
	void FRigidClustering::UpdateConnectivityGraphUsingDelaunayTriangulation(
		const TArray<FPBDRigidParticleHandle*>& Particles,
		const FClusterCreationParameters& Parameters,
		const TSet<FPBDRigidParticleHandle*>* FromParticles,
		const TSet<FPBDRigidParticleHandle*>* ToParticles)
	{
		SCOPE_CYCLE_COUNTER(STAT_UpdateConnectivityGraphUsingDelaunayTriangulation);

		if (UseBoundingBoxForConnectionGraphFiltering)
		{
			constexpr auto IsOverlappingConnectionUsingCVarMargin =
				[](const FPBDRigidParticleHandle* Child1, const FPBDRigidParticleHandle* Child2)
			{
				return IsOverlappingConnection(Child1, Child2, BoundingBoxMarginForConnectionGraphFiltering);
			};
			UpdateConnectivityGraphUsingDelaunayTriangulationWithFiltering(Particles, IsOverlappingConnectionUsingCVarMargin, FromParticles, ToParticles);
		}
		else
		{
			UpdateConnectivityGraphUsingDelaunayTriangulationWithFiltering(Particles, IsAlwaysValidConnection, FromParticles, ToParticles);
		}
	}

	void FRigidClustering::UpdateConnectivityGraphUsingDelaunayTriangulation(
		const Chaos::FPBDRigidClusteredParticleHandle* Parent, 
		const FClusterCreationParameters& Parameters)
	{
		const TArray<FPBDRigidParticleHandle*>& Children = MChildren[Parent];
		FixConnectivityGraphUsingDelaunayTriangulation(Children, Parameters);
	}

	DECLARE_CYCLE_STAT(TEXT("TPBDRigidClustering<>::UpdateConnectivityGraphUsingDelaunayTriangulationWithBoundsOverlaps"), STAT_UpdateConnectivityGraphUsingDelaunayTriangulationWithBoundsOverlaps, STATGROUP_Chaos);
	void FRigidClustering::UpdateConnectivityGraphUsingDelaunayTriangulationWithBoundsOverlaps(
		const TArray<FPBDRigidParticleHandle*>& Particles,
		const FClusterCreationParameters& Parameters,
		const TSet<FPBDRigidParticleHandle*>* FromParticles,
		const TSet<FPBDRigidParticleHandle*>* ToParticles)
	{
		SCOPE_CYCLE_COUNTER(STAT_UpdateConnectivityGraphUsingDelaunayTriangulationWithBoundsOverlaps);

		const auto IsOverlappingConnectionUsingCVarMargin =
			[&Parameters](const FPBDRigidParticleHandle* Child1, const FPBDRigidParticleHandle* Child2)
		{
			return IsOverlappingConnection(Child1, Child2, Parameters.ConnectionGraphBoundsFilteringMargin);
		};

		UpdateConnectivityGraphUsingDelaunayTriangulationWithFiltering(Particles, IsOverlappingConnectionUsingCVarMargin, FromParticles, ToParticles);
	}

	void FRigidClustering::UpdateConnectivityGraphUsingDelaunayTriangulationWithBoundsOverlaps(
		const Chaos::FPBDRigidClusteredParticleHandle* Parent, 
		const FClusterCreationParameters& Parameters)
	{
		const TArray<FPBDRigidParticleHandle*>& Children = MChildren[Parent];
		UpdateConnectivityGraphUsingDelaunayTriangulationWithBoundsOverlaps(Children, Parameters);
	}

	void 
	FRigidClustering::RemoveNodeConnections(
		FPBDRigidParticleHandle* Child)
	{
		RemoveNodeConnections(Child->CastToClustered());
	}

	DECLARE_CYCLE_STAT(TEXT("TPBDRigidClustering<>::RemoveNodeConnections"), STAT_RemoveNodeConnections, STATGROUP_Chaos);
	void 
	FRigidClustering::RemoveNodeConnections(
		FPBDRigidClusteredParticleHandle* ClusteredChild)
	{
		SCOPE_CYCLE_COUNTER(STAT_RemoveNodeConnections);
		RemoveFilteredNodeConnections(ClusteredChild, true);
	}

	bool FRigidClustering::ShouldThrottleParticleRelease() const
	{
		return (DestructionSettings.ClusteringParticleReleaseThrottlingMinCount >= 0 && DestructionSettings.ClusteringParticleReleaseThrottlingMaxCount >= 0);
	}

	void FRigidClustering::ThrottleReleasedParticlesIfNecessary(TSet<FPBDRigidParticleHandle*>& Particles)
	{
		GenericThrottleReleasedParticlesIfNecessary(Particles, *this);
	}

	void FRigidClustering::ThrottleReleasedParticlesIfNecessary(TArray<FPBDRigidParticleHandle*>& Particles)
	{
		GenericThrottleReleasedParticlesIfNecessary(Particles, *this);
	}

	FClusterCreationParameters::EConnectionMethod ToInternalConnectionMethod(EClusterUnionMethod InMethod)
	{
		using ETargetEnum = FClusterCreationParameters::EConnectionMethod;
		switch (InMethod)
		{
		case EClusterUnionMethod::PointImplicit:
			return ETargetEnum::PointImplicit;
		case EClusterUnionMethod::DelaunayTriangulation:
			return ETargetEnum::DelaunayTriangulation;
		case EClusterUnionMethod::MinimalSpanningSubsetDelaunayTriangulation:
			return ETargetEnum::MinimalSpanningSubsetDelaunayTriangulation;
		case EClusterUnionMethod::PointImplicitAugmentedWithMinimalDelaunay:
			return ETargetEnum::PointImplicitAugmentedWithMinimalDelaunay;
		case EClusterUnionMethod::BoundsOverlapFilteredDelaunayTriangulation:
			return ETargetEnum::BoundsOverlapFilteredDelaunayTriangulation;
		}

		return ETargetEnum::None;
	}

	void FRigidClustering::ApplySettings(const FChaosSolverConfiguration& Settings)
	{
		SetClusterConnectionFactor(Settings.ClusterConnectionFactor);
		SetClusterUnionConnectionType(ToInternalConnectionMethod(Settings.ClusterUnionConnectionType));
		DestructionSettings.PerAdvanceBreaksAllowed = FMath::Min(Settings.DestructionSettings.PerAdvanceBreaksAllowed, GPerAdvanceBreaksAllowed);
		DestructionSettings.PerAdvanceBreaksRescheduleLimit = FMath::Min(Settings.DestructionSettings.PerAdvanceBreaksRescheduleLimit, GPerAdvanceBreaksRescheduleLimit);
		DestructionSettings.ClusteringParticleReleaseThrottlingMinCount = FMath::Max(Settings.DestructionSettings.ClusteringParticleReleaseThrottlingMinCount, ClusteringParticleReleaseThrottlingMinCount);
		DestructionSettings.ClusteringParticleReleaseThrottlingMaxCount = FMath::Max(Settings.DestructionSettings.ClusteringParticleReleaseThrottlingMaxCount, ClusteringParticleReleaseThrottlingMaxCount);
	}

} // namespace Chaos
