// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/PBDRigidClusteringAlgo.h"

#include "Chaos/ErrorReporter.h"
#include "Chaos/Levelset.h"
#include "Chaos/Utilities.h"
#include "Chaos/PBDRigidClusteringCollisionParticleAlgo.h"
#include "Chaos/PBDRigidsEvolutionGBF.h"
#include "Chaos/MassProperties.h"

namespace Chaos
{
	//
    // Update Geometry PVar
	//

	int32 UseLevelsetCollision = 0;
	FAutoConsoleVariableRef CVarUseLevelsetCollision2(TEXT("p.UseLevelsetCollision"), UseLevelsetCollision, TEXT("Whether unioned objects use levelsets"));

	int32 MinLevelsetDimension = 4;
	FAutoConsoleVariableRef CVarMinLevelsetDimension2(TEXT("p.MinLevelsetDimension"), MinLevelsetDimension, TEXT("The minimum number of cells on a single level set axis"));

	int32 MaxLevelsetDimension = 20;
	FAutoConsoleVariableRef CVarMaxLevelsetDimension2(TEXT("p.MaxLevelsetDimension"), MaxLevelsetDimension, TEXT("The maximum number of cells on a single level set axis"));

	FRealSingle MinLevelsetSize = 50.f;
	FAutoConsoleVariableRef CVarLevelSetResolution2(TEXT("p.MinLevelsetSize"), MinLevelsetSize, TEXT("The minimum size on the smallest axis to use a level set"));

	int32 LevelsetGhostCells = 1;
	FAutoConsoleVariableRef CVarLevelsetGhostCells2(TEXT("p.LevelsetGhostCells"), LevelsetGhostCells, TEXT("Increase the level set grid by this many ghost cells"));

	int32 MinCleanedPointsBeforeRemovingInternals = 10;
	FAutoConsoleVariableRef CVarMinCleanedPointsBeforeRemovingInternals2(TEXT("p.MinCleanedPointsBeforeRemovingInternals"), MinCleanedPointsBeforeRemovingInternals, TEXT("If we only have this many clean points, don't bother removing internal points as the object is likely very small"));

	FRealSingle ClusterSnapDistance = 1.f;
	FAutoConsoleVariableRef CVarClusterSnapDistance2(TEXT("p.ClusterSnapDistance"), ClusterSnapDistance, TEXT(""));

	int32 GUseLockedChildToParentInMassProps = 0;
	FAutoConsoleVariableRef CVarUseLockedChildToParentInMassProps(
		TEXT("p.Chaos.ClusterUnion.UseLockedChildToParentInMassProps"),
		GUseLockedChildToParentInMassProps,
		TEXT("If 1, UpdateClusterMassProperties uses the stored ChildToParent for locked children instead of deriving it from cached world XR. Fixes a wrong-CoM bug when the parent XR has been updated without a corresponding child re-enforcement (same-tick parent push + child Remove, or free dynamic simulation of the parent)."));

	DECLARE_CYCLE_STAT(TEXT("TPBDRigidClustering<>::UpdateClusterMassProperties()"), STAT_UpdateClusterMassProperties, STATGROUP_Chaos);
	void UpdateClusterMassProperties(
		FPBDRigidClusteredParticleHandle* Parent,
		const TSet<FPBDRigidParticleHandle*>& Children)
	{
		SCOPE_CYCLE_COUNTER(STAT_UpdateClusterMassProperties);

		// NOTE: Default initializer will have zero mass
		FMassProperties ParentMass;

		// Combine mass properties of all children in parent's space
		if (Children.Num() > 0)
		{
			const FRigidTransform3 ParentTM = Parent->GetTransformXR();
			TArray<FMassProperties> ChildMasses;
			ChildMasses.Reserve(Children.Num());
			for (const FPBDRigidParticleHandle* Child : Children)
			{
				// For locked children, ChildToParent is authoritative; use it directly
				// rather than re-deriving from cached world XR, which may be stale relative
				// to a parent XR that was just updated by simulation, replication, or
				// correction without a corresponding child re-enforcement.
				FRigidTransform3 ChildToParentTM;
				const FPBDRigidClusteredParticleHandle* ClusteredChild = Child->CastToClustered();
				if (GUseLockedChildToParentInMassProps && ClusteredChild && ClusteredChild->IsChildToParentLocked())
				{
					ChildToParentTM = ClusteredChild->ChildToParent();
				}
				else
				{
					const FRigidTransform3 ChildTM = Child->GetTransformXR();
					ChildToParentTM = ChildTM.GetRelativeTransform(ParentTM);
				}

				const FRigidTransform3 ChildCoM(Child->CenterOfMass(), Child->RotationOfMass());
				const FRigidTransform3 ChildCoMInParentSpace = ChildCoM * ChildToParentTM;

				// Get the child's mass properties
				FMassProperties ChildMass;
				ChildMass.Mass = Child->M();
				ChildMass.InertiaTensor = FMatrix33(Child->I());
				ChildMass.CenterOfMass = ChildCoMInParentSpace.GetTranslation();
				ChildMass.RotationOfMass = ChildCoMInParentSpace.GetRotation();
				ChildMasses.Add(ChildMass);
			}

			ParentMass = Chaos::Combine(ChildMasses);
		}

		// Set properties of particle based on combined mass properties
		// NOTE: The combine method will have diagonalized the inertia.
		const FVec3 Inertia = ParentMass.InertiaTensor.GetDiagonal();
		Parent->SetCenterOfMass(ParentMass.CenterOfMass);
		Parent->SetRotationOfMass(ParentMass.RotationOfMass);
		Parent->M() = ParentMass.Mass;
		Parent->InvM()
			= FMath::IsNearlyZero(ParentMass.Mass)
			? 0.f
			: 1.f / ParentMass.Mass;
		Parent->I() = Inertia;
		Parent->InvI()
			= (FMath::IsNearlyZero(Inertia[0]) || FMath::IsNearlyZero(Inertia[1]) || FMath::IsNearlyZero(Inertia[2]))
			? FVec3::ZeroVector
			: FReal(1) / Inertia;
	}

	void AdjustClusterInertia(FPBDRigidClusteredParticleHandle* Cluster, const EInertiaOperations InertiaOperations)
	{
		if ((InertiaOperations & EInertiaOperations::LocalInertiaDropOffDiagonalTerms) != EInertiaOperations::None)
		{
			// Discard off-diagonal terms in inertia transformed into a particle's local space, and
			// zero out the Rotation of Inertia parameter.
			const FMatrix33 LocalSpaceInertia = Utilities::ComputeWorldSpaceInertia(Cluster->RotationOfMass(), Cluster->I());
			Cluster->SetI(LocalSpaceInertia.GetDiagonal());
			Cluster->SetRotationOfMass(FQuat::Identity);
		}
	}

	FRigidTransform3 MoveClusterToMassOffset(FPBDRigidClusteredParticleHandle* Cluster, const EMassOffsetType MassOffsetTypes)
	{
		FRigidTransform3 MassToLocal = FRigidTransform3::Identity;

		if ((MassOffsetTypes & EMassOffsetType::Position) != EMassOffsetType::None)
		{
			MassToLocal.SetTranslation(Cluster->CenterOfMass());
			Cluster->SetX(Cluster->XCom());
			Cluster->SetP(Cluster->GetX());
			Cluster->SetCenterOfMass(FVec3::ZeroVector);
		}

		if ((MassOffsetTypes & EMassOffsetType::Rotation) != EMassOffsetType::None)
		{
			MassToLocal.SetRotation(Cluster->RotationOfMass());
			Cluster->SetR(Cluster->RCom());
			Cluster->SetQf(Cluster->GetRf());
			Cluster->SetRotationOfMass(FQuat::Identity);
		}

		return MassToLocal;
	}

	DECLARE_CYCLE_STAT(TEXT("TPBDRigidClustering<>::UpdateKinematicProperties()"), STAT_UpdateKinematicProperties, STATGROUP_Chaos);
	void 
		UpdateKinematicProperties(
			Chaos::FPBDRigidParticleHandle* InParent,
			const FRigidClustering::FClusterMap& MChildren,
			FRigidClustering::FRigidEvolution& MEvolution)
	{
		SCOPE_CYCLE_COUNTER(STAT_UpdateKinematicProperties);
		typedef FPBDRigidClusteredParticleHandle* FClusterHandle;

		// preserve a previous sleeping state 
		const bool bWasSleeping = InParent && InParent->IsSleeping();

		EObjectStateType ObjectState = EObjectStateType::Dynamic;
		check(InParent != nullptr);
		if (FClusterHandle ClusteredCurrentNode = InParent->CastToClustered())
		{
			if (MChildren.Contains(ClusteredCurrentNode) && MChildren[ClusteredCurrentNode].Num())
			{
				if (ClusteredCurrentNode->IsAnchored())
				{
					ObjectState = EObjectStateType::Kinematic;
				}
				else
				{
					// TQueue is a linked list, which has no preallocator.
					TQueue<Chaos::FPBDRigidParticleHandle*> Queue;
					for (Chaos::FPBDRigidParticleHandle* Child : MChildren[ClusteredCurrentNode])
					{
						Queue.Enqueue(Child);
					}

					Chaos::FPBDRigidParticleHandle* CurrentHandle;
					while (Queue.Dequeue(CurrentHandle) && ObjectState == EObjectStateType::Dynamic)
					{
						bool bIsAnchored = false;
						if (FClusterHandle CurrentClusterHandle = CurrentHandle->CastToClustered())
						{
							// @question : Maybe we should just store the leaf node bodies in a
							// map, that will require Memory(n*log(n))
							if (MChildren.Contains(CurrentClusterHandle))
							{
								for (Chaos::FPBDRigidParticleHandle* Child : MChildren[CurrentClusterHandle])
								{
									Queue.Enqueue(Child);
								}
							}
							
							bIsAnchored = CurrentClusterHandle->IsAnchored();
						}

						if (bIsAnchored)
						{
							ObjectState = EObjectStateType::Kinematic;
						}
						else
						{
							const EObjectStateType CurrState = CurrentHandle->ObjectState();
							if (CurrState == EObjectStateType::Kinematic)
							{
								ObjectState = EObjectStateType::Kinematic;
							}
							else if (CurrState == EObjectStateType::Static)
							{
								ObjectState = EObjectStateType::Static;
							}
						}
					}
				}

				if (bWasSleeping && ObjectState == Chaos::EObjectStateType::Dynamic)
				{
					ObjectState = Chaos::EObjectStateType::Sleeping;
				}

				MEvolution.SetParticleObjectState(ClusteredCurrentNode, ObjectState);
				if (ObjectState != Chaos::EObjectStateType::Kinematic)
				{
					MEvolution.SetParticleKinematicTarget(ClusteredCurrentNode, FKinematicTarget());
				}
			}
		}
	}
	

	DECLARE_CYCLE_STAT(TEXT("TPBDRigidClustering<>::UpdateGeometry"), STAT_UpdateGeometry, STATGROUP_Chaos);
	DECLARE_CYCLE_STAT(TEXT("TPBDRigidClustering<>::UpdateGeometry_GatherObjects"), STAT_UpdateGeometry_GatherObjects, STATGROUP_Chaos);
	DECLARE_CYCLE_STAT(TEXT("TPBDRigidClustering<>::UpdateGeometry_GatherPoints"), STAT_UpdateGeometry_GatherPoints, STATGROUP_Chaos);
	DECLARE_CYCLE_STAT(TEXT("TPBDRigidClustering<>::UpdateGeometry_CopyPoints"), STAT_UpdateGeometry_CopyPoints, STATGROUP_Chaos);
	DECLARE_CYCLE_STAT(TEXT("TPBDRigidClustering<>::UpdateGeometry_PointsBVH"), STAT_UpdateGeometry_PointsBVH, STATGROUP_Chaos);

	void BuildScaledGeometry(Chaos::FPBDRigidClusteredParticleHandle* ParticleHandle, const Chaos::FImplicitObjectPtr& ProxyGeometry, const FVector& WorldScale)
	{
		if(ParticleHandle && ProxyGeometry.IsValid())
		{ 
			auto DeepCopyImplicit = [&WorldScale](const Chaos::FImplicitObjectPtr& ImplicitToCopy) -> Chaos::FImplicitObjectPtr
			{
				if (WorldScale.Equals(FVector::OneVector))
				{
					return ImplicitToCopy->DeepCopyGeometry();
				}
				else
				{
					return ImplicitToCopy->DeepCopyGeometryWithScale(WorldScale);
				}
			};

			Chaos::EImplicitObjectType GeometryType = ProxyGeometry->GetType();
			// Don't copy if it is not a level set and scale is one
			if (GeometryType != Chaos::ImplicitObjectType::LevelSet && WorldScale.Equals(FVector::OneVector))
			{
				ParticleHandle->SetGeometry(ProxyGeometry);
			}
			else
			{
				ParticleHandle->SetGeometry(DeepCopyImplicit(ProxyGeometry));
			}
		}
	}

	void UpdateCollisionFlags(Chaos::FPBDRigidClusteredParticleHandle* ParticleHandle, const bool bUseParticleImplicit)
	{
		if (FImplicitObjectPtr ImplicitGeometry = ParticleHandle->GetGeometry())
		{
			// if children are ignore analytic and this is a dynamic geom, mark it too. todo(ocohen): clean this up
			if (bUseParticleImplicit)
			{
				ImplicitGeometry->SetDoCollide(false);
			}

			ParticleHandle->SetHasBounds(true);
			ParticleHandle->SetLocalBounds(ImplicitGeometry->BoundingBox());

			if (const FImplicitObjectUnion* ImplicitUnion = ImplicitGeometry->GetObject<FImplicitObjectUnion>())
			{
				const_cast<FImplicitObjectUnion*>(ImplicitUnion)->SetAllowBVH(true);
			}
			else if (const FImplicitObjectUnion* ImplicitUnionClustered = ImplicitGeometry->GetObject<FImplicitObjectUnionClustered>())
			{
				const_cast<FImplicitObjectUnion*>(ImplicitUnionClustered)->SetAllowBVH(true);
			}
		}
	}

	void
	UpdateGeometry(
		Chaos::FPBDRigidClusteredParticleHandle* Parent,
		const TSet<FPBDRigidParticleHandle*>& Children,
		const FRigidClustering::FClusterMap& ChildrenMap,
		const Chaos::FImplicitObjectPtr& ProxyGeometry,
		const FClusterCreationParameters& Parameters)
	{
		SCOPE_CYCLE_COUNTER(STAT_UpdateGeometry);

		TArray<Chaos::FImplicitObjectPtr> Objects;
		TArray<Chaos::FImplicitObjectPtr> Objects2; //todo: find a better way to reuse this
		Objects.Reserve(Children.Num());
		Objects2.Reserve(Children.Num());

		const FRigidTransform3 ClusterWorldTM(Parent->GetX(), Parent->GetR());

		TArray<FVec3> OriginalPoints;
		TArray<FPBDRigidParticleHandle*> ChildParticleHandles;
		ChildParticleHandles.Reserve(Children.Num());

		const bool bUseCollisionPoints = (ProxyGeometry || Parameters.bCopyCollisionParticles) && !Parameters.CollisionParticles;
		bool bUseParticleImplicit = false;

		{ // STAT_UpdateGeometry_GatherObjects
			SCOPE_CYCLE_COUNTER(STAT_UpdateGeometry_GatherObjects);

			if (bUseCollisionPoints)
			{
				uint32 NumPoints = 0;
				for (FPBDRigidParticleHandle* Child : Children)
				{
					NumPoints += Child->CollisionParticlesSize();
				}
				OriginalPoints.Reserve(NumPoints);
			}

			for (FPBDRigidParticleHandle* Child : Children)
			{
				FRigidTransform3 Frame = FRigidTransform3::Identity;

				if (FPBDRigidClusteredParticleHandle* ClusterChild = Child->CastToClustered(); ClusterChild && ClusterChild->IsChildToParentLocked())
				{
					Frame = ClusterChild->ChildToParent();
				}
				else
				{
					const FRigidTransform3 ChildWorldTM(Child->GetX(), Child->GetR());
					Frame = ChildWorldTM.GetRelativeTransform(ClusterWorldTM);
				}

				FPBDRigidParticleHandle* UsedGeomChild = Child;
				if (Child->GetGeometry())
				{
					Objects.Add(MakeImplicitObjectPtr<TImplicitObjectTransformed<FReal, 3>>(Child->GetGeometry(), Frame));
					Objects2.Add(MakeImplicitObjectPtr<TImplicitObjectTransformed<FReal, 3>>(Child->GetGeometry(), Frame));
					ChildParticleHandles.Add(Child);
				}

				ensure(Child->Disabled() == true);
				check(Child->CastToClustered()->ClusterIds().Id == Parent);

				Child->CastToClustered()->SetChildToParent(Frame);

				if (bUseCollisionPoints)
				{
					SCOPE_CYCLE_COUNTER(STAT_UpdateGeometry_GatherPoints);
					if (const TUniquePtr<FBVHParticles>& CollisionParticles = Child->CollisionParticles())
					{
						for (uint32 i = 0; i < CollisionParticles->Size(); ++i)
						{
							OriginalPoints.Add(Frame.TransformPosition(CollisionParticles->GetX(i)));
						}
					}
				}
				if (Child->GetGeometry() && Child->GetGeometry()->GetType() == ImplicitObjectType::Unknown)
				{
					bUseParticleImplicit = true;
				}
			} // end for
		} // STAT_UpdateGeometry_GatherObjects

		{
			QUICK_SCOPE_CYCLE_COUNTER(SpatialBVH);
			FImplicitObjectUnionClusteredPtr& ChildrenSpatial = Parent->GetChildrenSpatial();
			FImplicitObjectUnionClustered* UnionClustered = Objects2.Num() ?
				new Chaos::FImplicitObjectUnionClustered(MoveTemp(Objects2), ChildParticleHandles) :
				nullptr;
			
			ChildrenSpatial = FImplicitObjectUnionClusteredPtr(UnionClustered);
		}

		TArray<FVec3> CleanedPoints;
		if (!Parameters.CollisionParticles)
		{
			CleanedPoints =
				Parameters.bCleanCollisionParticles ?
				CleanCollisionParticles(OriginalPoints, ClusterSnapDistance) :
				OriginalPoints;
		}

		// ignore unions for now as we don't yet support deep copy of it
		// on the GT they are only used by clusters that aggregate their children shapes ( see GeometryCollectionPhysicsProxy.cpp )
		// by failing artificially this condition thmake sure we create a FImplicitObjectUnionClustered for this particle 
		if (ProxyGeometry)
		{
			BuildScaledGeometry(Parent, ProxyGeometry, Parameters.Scale);
		}
		else if (Objects.Num() == 0)
		{
			//ensureMsgf(false, TEXT("Checking usage with no proxy and no objects"));
			//@coverage : {production}
			Parent->SetGeometry(Chaos::FImplicitObjectPtr());
		}
		else
		{
			if (UseLevelsetCollision)
			{
				ensureMsgf(false, TEXT("Checking usage with no proxy and multiple ojects with levelsets"));

				FImplicitObjectUnionClustered UnionObject(MoveTemp(Objects));
				FAABB3 Bounds = UnionObject.BoundingBox();
				const FVec3 BoundsExtents = Bounds.Extents();
				if (BoundsExtents.Min() >= MinLevelsetSize) //make sure the object is not too small
				{
					TVec3<int32> NumCells = Bounds.Extents() / MinLevelsetSize;
					for (int i = 0; i < 3; ++i)
					{
						NumCells[i] = FMath::Clamp(NumCells[i], MinLevelsetDimension, MaxLevelsetDimension);
					}

					FErrorReporter ErrorReporter;
					TUniformGrid<FReal, 3> Grid(Bounds.Min(), Bounds.Max(), NumCells, LevelsetGhostCells);
					FLevelSet* LevelSet = new FLevelSet(ErrorReporter, Grid, UnionObject);

					if (!Parameters.CollisionParticles)
					{
						const FReal MinDepthToSurface = Grid.Dx().Max();
						for (int32 Idx = CleanedPoints.Num() - 1; Idx >= 0; --Idx)
						{
							if (CleanedPoints.Num() > MinCleanedPointsBeforeRemovingInternals) //todo(ocohen): this whole thing should really be refactored
							{
								const FVec3& CleanedCollision = CleanedPoints[Idx];
								if (LevelSet->SignedDistance(CleanedCollision) < -MinDepthToSurface)
								{
									CleanedPoints.RemoveAtSwap(Idx);
								}
							}
						}
					}
					Chaos::FImplicitObjectPtr LevelSetPtr(LevelSet);
					Parent->SetGeometry(MoveTemp(LevelSetPtr));
				}
				else
				{
					Parent->SetGeometry(MakeImplicitObjectPtr<FSphere>(FVec3(0), BoundsExtents.Size() * 0.5f));
				}
			}
			else // !UseLevelsetCollision
			{
				if (Objects.Num() == 1)
				{
					Parent->SetGeometry(MoveTemp(Objects[0]));
				}
				else
				{
					Parent->SetGeometry(
						MakeImplicitObjectPtr<FImplicitObjectUnionClustered>(
							MoveTemp(Objects), ChildParticleHandles));
				}
			}
		}

		
		if (Parameters.CollisionParticles)
		{
			SCOPE_CYCLE_COUNTER(STAT_UpdateGeometry_CopyPoints);
			Parent->CollisionParticles().Reset(Parameters.CollisionParticles);
		}
		else
		{
			{
				SCOPE_CYCLE_COUNTER(STAT_UpdateGeometry_GatherPoints);
				Parent->CollisionParticlesInitIfNeeded();
				TUniquePtr<FBVHParticles>& CollisionParticles = Parent->CollisionParticles();
				CollisionParticles->AddParticles(CleanedPoints.Num());
				for (int32 i = 0; i < CleanedPoints.Num(); ++i)
				{
					CollisionParticles->SetX(i, CleanedPoints[i]);
				}
			}

			if (bUseCollisionPoints)
			{
				SCOPE_CYCLE_COUNTER(STAT_UpdateGeometry_PointsBVH);
				Parent->CollisionParticles()->UpdateAccelerationStructures();
			}
		}
		const Chaos::FRigidTransform3 Xf(Parent->GetX(), Parent->GetR());
		Parent->UpdateWorldSpaceState(Xf, FVec3(0));

		UpdateCollisionFlags(Parent, bUseParticleImplicit);
	
		// Update filter data on new shapes
		const FRigidClustering::FRigidHandleArray& ChildrenArray = ChildrenMap[Parent];
		UpdateClusterFilterDataFromChildren(Parent, ChildrenArray);
	}
	
	void UpdateClusterFilterDataFromChildren(FPBDRigidClusteredParticleHandle* ClusterParent, const TArray<FPBDRigidParticleHandle*>& Children)
	{
		SCOPE_CYCLE_COUNTER(STAT_GCUpdateFilterData);

		Filter::FCombinedShapeFilterData SelectedShapeFilter;
		bool bFilterValid = false;
		for (FPBDRigidParticleHandle* ChildHandle : Children)
		{
			for (const TUniquePtr<FPerShapeData>& Shape : ChildHandle->ShapesArray())
			{
				if (Shape)
				{
					SelectedShapeFilter = Shape->GetCombinedShapeFilterData();
					bFilterValid = SelectedShapeFilter.IsSimValid();

					if (bFilterValid)
					{
						break;
					}
				}
			}

			if (bFilterValid)
			{
				break;
			}
		}

		// Apply selected filters to shapes
		if (bFilterValid)
		{
			const FShapesArray& ShapesArray = ClusterParent->ShapesArray();
			for (const TUniquePtr<FPerShapeData>& Shape : ShapesArray)
			{
				Shape->SetCombinedShapeFilterData(SelectedShapeFilter);
			}
		}
	}


} // namespace Chaos
