// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/PBDRigidDynamicSpringConstraints.h"
#include "Chaos/Island/IslandManager.h"
#include "Chaos/Particle/ParticleUtilities.h"
#include "Chaos/Utilities.h"

namespace Chaos
{
	FParticlePair FPBDRigidDynamicSpringConstraintHandle::GetConstrainedParticles() const
	{
		return ConcreteContainer()->GetConstrainedParticles(ConstraintIndex);
	}

	// Note: this is called outside the constraint solver loop (it generates constraints and can merge 
	// islands - the equivalent of collision detection) and therefore reads directly from Particles, 
	// rather than from SolverBodies.
	void FPBDRigidDynamicSpringConstraints::UpdatePositionBasedState(const FReal Dt)
	{
		const int32 NumConstraints = Constraints.Num();
		for(int32 ConstraintIndex = 0; ConstraintIndex < NumConstraints; ++ConstraintIndex)
		{
			FGeometryParticleHandle* Static0 = Constraints[ConstraintIndex][0];
			FGeometryParticleHandle* Static1 = Constraints[ConstraintIndex][1];
			FPBDRigidParticleHandle* PBDRigid0 = Static0->CastToRigidParticle();
			FPBDRigidParticleHandle* PBDRigid1 = Static1->CastToRigidParticle();
			const bool bIsRigidDynamic0 = PBDRigid0 && PBDRigid0->ObjectState() == EObjectStateType::Dynamic;
			const bool bIsRigidDynamic1 = PBDRigid1 && PBDRigid1->ObjectState() == EObjectStateType::Dynamic;

			// Do not create springs between objects with no geometry
			if(!Static0->GetGeometry() || !Static1->GetGeometry())
			{
				continue;
			}

			const FRotation3 Q0 = bIsRigidDynamic0 ? PBDRigid0->GetQ() : Static0->GetR();
			const FRotation3 Q1 = bIsRigidDynamic1 ? PBDRigid1->GetQ() : Static1->GetR();
			const FVec3 P0 = bIsRigidDynamic0 ? PBDRigid0->GetP() : Static0->GetX();
			const FVec3 P1 = bIsRigidDynamic1 ? PBDRigid1->GetP() : Static1->GetX();

			// Delete constraints
			const int32 NumSprings = SpringDistances[ConstraintIndex].Num();
			for(int32 SpringIndex = NumSprings - 1; SpringIndex >= 0; --SpringIndex)
			{
				const FVec3& Distance0 = Distances[ConstraintIndex][SpringIndex][0];
				const FVec3& Distance1 = Distances[ConstraintIndex][SpringIndex][1];
				const FVec3 WorldSpaceX1 = Q0.RotateVector(Distance0) + P0;
				const FVec3 WorldSpaceX2 = Q1.RotateVector(Distance1) + P1;
				const FVec3 Difference = WorldSpaceX2 - WorldSpaceX1;
				FReal Distance = Difference.Size();
				if(Distance > CreationThreshold * 2)
				{
					Distances[ConstraintIndex].RemoveAtSwap(SpringIndex);
					SpringDistances[ConstraintIndex].RemoveAtSwap(SpringIndex);
				}
			}

			if(SpringDistances[ConstraintIndex].Num() == MaxSprings)
			{
				continue;
			}

			FRigidTransform3 Transform1(P0, Q0);
			FRigidTransform3 Transform2(P1, Q1);

			// Create constraints
			if(Static0->GetGeometry()->HasBoundingBox() && Static1->GetGeometry()->HasBoundingBox())
			{
				// Matrix multiplication is reversed intentionally to be compatible with unreal
				FAABB3 Box1 = Static0->GetGeometry()->BoundingBox().TransformedAABB(Transform1 * Transform2.Inverse());
				Box1.Thicken(CreationThreshold);
				FAABB3 Box2 = Static1->GetGeometry()->BoundingBox();
				Box2.Thicken(CreationThreshold);
				if(!Box1.Intersects(Box2))
				{
					continue;
				}
			}
			const FVec3 Midpoint = (P0 + P1) / (FReal)2.;
			FVec3 Normal1;
			const FReal Phi1 = Static0->GetGeometry()->PhiWithNormal(Transform1.InverseTransformPosition(Midpoint), Normal1);
			Normal1 = Transform2.TransformVector(Normal1);
			FVec3 Normal2;
			const FReal Phi2 = Static1->GetGeometry()->PhiWithNormal(Transform2.InverseTransformPosition(Midpoint), Normal2);
			Normal2 = Transform2.TransformVector(Normal2);
			if((Phi1 + Phi2) > CreationThreshold)
			{
				continue;
			}
			FVec3 Location0 = Midpoint - Phi1 * Normal1;
			FVec3 Location1 = Midpoint - Phi2 * Normal2;
			TVec2<FVec3> Distance;
			Distance[0] = Q0.Inverse().RotateVector(Location0 - P0);
			Distance[1] = Q0.Inverse().RotateVector(Location1 - P1);
			Distances[ConstraintIndex].Add(MoveTemp(Distance));
			SpringDistances[ConstraintIndex].Add((Location0 - Location1).Size());
		}
	}

	void FPBDRigidDynamicSpringConstraints::AddConstraintsToGraph(Private::FPBDIslandManager& IslandManager)
	{
		IslandManager.AddContainerConstraints(*this);
	}

	void FPBDRigidDynamicSpringConstraints::AddBodies(FSolverBodyContainer& SolverBodyContainer)
	{
		for(int32 ConstraintIndex = 0; ConstraintIndex < NumConstraints(); ++ConstraintIndex)
		{
			AddBodies(ConstraintIndex, SolverBodyContainer);
		}
	}

	void FPBDRigidDynamicSpringConstraints::ScatterOutput(const FReal Dt)
	{
		for(int32 ConstraintIndex = 0; ConstraintIndex < NumConstraints(); ++ConstraintIndex)
		{
			ConstraintSolverBodies[ConstraintIndex] =
			{
				nullptr,
				nullptr
			};
		}
	}

	void FPBDRigidDynamicSpringConstraints::ApplyPositionConstraints(const FReal Dt, const int32 It, const int32 NumIts)
	{
		for(int32 ConstraintIndex = 0; ConstraintIndex < NumConstraints(); ++ConstraintIndex)
		{
			ApplySingle(Dt, ConstraintIndex);
		}
	}

	void FPBDRigidDynamicSpringConstraints::AddBodies(const TArrayView<int32>& ConstraintIndices, FSolverBodyContainer& SolverBodyContainer)
	{
		for(int32 ConstraintIndex : ConstraintIndices)
		{
			AddBodies(ConstraintIndex, SolverBodyContainer);
		}
	}

	void FPBDRigidDynamicSpringConstraints::ScatterOutput(const TArrayView<int32>& ConstraintIndices, const FReal Dt)
	{
		for(int32 ConstraintIndex : ConstraintIndices)
		{
			ConstraintSolverBodies[ConstraintIndex] =
			{
				nullptr,
				nullptr
			};
		}
	}

	void FPBDRigidDynamicSpringConstraints::ApplyPositionConstraints(const TArrayView<int32>& ConstraintIndices, const FReal Dt, const int32 It, const int32 NumIts)
	{
		for(int32 ConstraintIndex : ConstraintIndices)
		{
			ApplySingle(Dt, ConstraintIndex);
		}
	}

	void FPBDRigidDynamicSpringConstraints::AddBodies(const int32 ConstraintIndex, FSolverBodyContainer& SolverBodyContainer)
	{
		ConstraintSolverBodies[ConstraintIndex] =
		{
			SolverBodyContainer.FindOrAdd(Constraints[ConstraintIndex][0]),
			SolverBodyContainer.FindOrAdd(Constraints[ConstraintIndex][1])
		};
	}

	FVec3 FPBDRigidDynamicSpringConstraints::GetDelta(const FVec3& WorldSpaceX1, const FVec3& WorldSpaceX2, const int32 ConstraintIndex, const int32 SpringIndex) const
	{
		FSolverBody& Body0 = *ConstraintSolverBodies[ConstraintIndex][0];
		FSolverBody& Body1 = *ConstraintSolverBodies[ConstraintIndex][1];
		check(Body0.IsDynamic() || Body1.IsDynamic());

		const FVec3 Difference = WorldSpaceX2 - WorldSpaceX1;
		FReal Distance = Difference.Size();
		check(Distance > 1e-7);

		const FVec3 Direction = Difference / Distance;
		const FVec3 Delta = (Distance - SpringDistances[ConstraintIndex][SpringIndex]) * Direction;
		return Stiffness * Delta / (Body0.InvM() + Body1.InvM());
	}

	void FPBDRigidDynamicSpringConstraints::ApplySingle(const FReal Dt, int32 ConstraintIndex) const
	{
		check(ConstraintSolverBodies[ConstraintIndex][0] != nullptr);
		check(ConstraintSolverBodies[ConstraintIndex][1] != nullptr);
		FSolverBody& Body0 = *ConstraintSolverBodies[ConstraintIndex][0];
		FSolverBody& Body1 = *ConstraintSolverBodies[ConstraintIndex][1];
		check(Body0.IsDynamic() || Body1.IsDynamic());

		const int32 NumSprings = SpringDistances[ConstraintIndex].Num();
		for(int32 SpringIndex = 0; SpringIndex < NumSprings; ++SpringIndex)
		{
			const FVec3& Distance0 = Distances[ConstraintIndex][SpringIndex][0];
			const FVec3& Distance1 = Distances[ConstraintIndex][SpringIndex][1];
			const FVec3 WorldSpaceX1 = Body0.CorrectedQ().RotateVector(Distance0) + Body0.CorrectedP();
			const FVec3 WorldSpaceX2 = Body1.CorrectedQ().RotateVector(Distance1) + Body1.CorrectedP();
			const FVec3 Delta = GetDelta(WorldSpaceX1, WorldSpaceX2, ConstraintIndex, SpringIndex);

			if(Body0.IsDynamic())
			{
				const FVec3 Radius = WorldSpaceX1 - Body0.P();
				const FVec3 DX = Body0.InvM() * Delta;
				const FVec3 DR = Body0.InvI() * FVec3::CrossProduct(Radius, Delta);
				Body0.ApplyTransformDelta(DX, DR);
				Body0.UpdateRotationDependentState();
			}

			if(Body1.IsDynamic())
			{
				const FVec3 Radius = WorldSpaceX2 - Body1.P();
				const FVec3 DX = Body1.InvM() * -Delta;
				const FVec3 DR = Body1.InvI() * FVec3::CrossProduct(Radius, -Delta);
				Body1.ApplyTransformDelta(DX, DR);
				Body1.UpdateRotationDependentState();
			}
		}
	}
}
