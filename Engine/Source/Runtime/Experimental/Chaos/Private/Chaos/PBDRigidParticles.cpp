// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/PBDRigidParticles.h"

void Chaos::EnsureSleepingObjectState(EObjectStateType ObjectState)
{
	// feasible for game to set a dynamic body to kinematic state then to sleeping state
	//ensure(ObjectState != EObjectStateType::Kinematic);
	ensure(ObjectState != EObjectStateType::Static);
}

template class Chaos::TPBDRigidParticles<Chaos::FReal, 3>;
