// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/PBDRigidSpringConstraints.h"
#include "Chaos/Island/IslandManager.h"
#include "Chaos/Particle/ParticleUtilities.h"
#include "Chaos/Utilities.h"

namespace Chaos
{
	//
	// Handle Impl
	//

	const TVector<FVec3, 2>& FPBDRigidSpringConstraintHandle::GetConstraintPositions() const
	{
		return ConcreteContainer()->GetConstraintPositions(ConstraintIndex);

	}

	void FPBDRigidSpringConstraintHandle::SetConstraintPositions(const TVector<FVec3, 2>& ConstraintPositions)
	{
		ConcreteContainer()->SetConstraintPositions(ConstraintIndex, ConstraintPositions);
	}

	FParticlePair FPBDRigidSpringConstraintHandle::GetConstrainedParticles() const
	{
		return ConcreteContainer()->GetConstrainedParticles(ConstraintIndex);
	}

	FReal FPBDRigidSpringConstraintHandle::GetRestLength() const
	{
		return ConcreteContainer()->GetRestLength(ConstraintIndex);
	}

	void FPBDRigidSpringConstraintHandle::SetRestLength(const FReal SpringLength)
	{
		ConcreteContainer()->SetRestLength(ConstraintIndex, SpringLength);
	}

	//
	// Container Impl
	//

	FPBDRigidSpringConstraints::FPBDRigidSpringConstraints()
		: TPBDIndexedConstraintContainer<FPBDRigidSpringConstraints>(FConstraintContainerHandle::StaticType())
	{}

	FPBDRigidSpringConstraints::~FPBDRigidSpringConstraints()
	{
	}

	typename FPBDRigidSpringConstraints::FConstraintContainerHandle* FPBDRigidSpringConstraints::AddConstraint(const FConstrainedParticlePair& InConstrainedParticles, const  TVector<FVec3, 2>& InLocations, FReal Stiffness, FReal Damping, FReal RestLength)
	{
		Handles.Add(HandleAllocator.AllocHandle(this, Handles.Num()));
		int32 ConstraintIndex = Constraints.Add(InConstrainedParticles);

		SpringSettings.Emplace(FSpringSettings({ Stiffness, Damping, RestLength }));

		Distances.Add({});
		InitDistance(ConstraintIndex, InLocations[0], InLocations[1]);

		ConstraintSolverBodies.Add({ nullptr, nullptr });

		return Handles.Last();
	}

	void FPBDRigidSpringConstraints::RemoveConstraint(int ConstraintIndex)
	{
		FConstraintContainerHandle* ConstraintHandle = Handles[ConstraintIndex];
		if (ConstraintHandle != nullptr)
		{
			// Release the handle for the freed constraint
			HandleAllocator.FreeHandle(ConstraintHandle);
			Handles[ConstraintIndex] = nullptr;
		}

		// Swap the last constraint into the gap to keep the array packed
		Constraints.RemoveAtSwap(ConstraintIndex);
		SpringSettings.RemoveAtSwap(ConstraintIndex);
		Distances.RemoveAtSwap(ConstraintIndex);
		ConstraintSolverBodies.RemoveAtSwap(ConstraintIndex);
		Handles.RemoveAtSwap(ConstraintIndex);

		// Update the handle for the constraint that was moved
		if (ConstraintIndex < Handles.Num())
		{
			SetConstraintIndex(Handles[ConstraintIndex], ConstraintIndex);
		}
	}

	void FPBDRigidSpringConstraints::InitDistance(int32 ConstraintIndex, const FVec3& Location0, const FVec3& Location1)
	{
		// \note: this is called during initialization, not suring the solve. It therefore accesses
		// the Particles directly, rather than the SolverBodies
		const TVector<TGeometryParticleHandle<FReal, 3>*, 2>& Constraint = Constraints[ConstraintIndex];
		const TGeometryParticleHandle<FReal, 3>* Particle0 = Constraint[0];
		const TGeometryParticleHandle<FReal, 3>* Particle1 = Constraint[1];

		Distances[ConstraintIndex][0] = Particle0->GetR().Inverse().RotateVector(Location0 - Particle0->GetX());
		Distances[ConstraintIndex][1] = Particle1->GetR().Inverse().RotateVector(Location1 - Particle1->GetX());
	}

	FVec3 FPBDRigidSpringConstraints::GetDelta(int32 ConstraintIndex, const FVec3& WorldSpaceX1, const FVec3& WorldSpaceX2) const
	{
		FSolverBody& Body0 = *ConstraintSolverBodies[ConstraintIndex][0];
		FSolverBody& Body1 = *ConstraintSolverBodies[ConstraintIndex][1];

		if (!Body0.IsDynamic() && !Body1.IsDynamic())
		{
			return FVec3(0);
		}

		const FVec3 Difference = WorldSpaceX2 - WorldSpaceX1;
		const FReal Distance = Difference.Size();
		if (Distance < FReal(1e-7))
		{
			return FVec3(0);
		}

		const FVec3 Direction = Difference / Distance;
		const FVec3 Delta = (Distance - SpringSettings[ConstraintIndex].RestLength) * Direction;
		const FReal CombinedInvMass = Body0.InvM() + Body1.InvM();

		return SpringSettings[ConstraintIndex].Stiffness * Delta / CombinedInvMass;
	}

	void FPBDRigidSpringConstraints::AddConstraintsToGraph(Private::FPBDIslandManager& IslandManager)
	{
		IslandManager.AddContainerConstraints(*this);
	}

	void FPBDRigidSpringConstraints::AddBodies(FSolverBodyContainer& SolverBodyContainer)
	{
		for (int32 ConstraintIndex = 0; ConstraintIndex < NumConstraints(); ++ConstraintIndex)
		{
			AddBodies(ConstraintIndex, SolverBodyContainer);
		}
	}

	void FPBDRigidSpringConstraints::ScatterOutput(const FReal Dt)
	{
		for (int32 ConstraintIndex = 0; ConstraintIndex < NumConstraints(); ++ConstraintIndex)
		{
			ConstraintSolverBodies[ConstraintIndex] =
			{
				nullptr,
				nullptr
			};
		}
	}

	void FPBDRigidSpringConstraints::ApplyPositionConstraints(const FReal Dt, const int32 It, const int32 NumIts)
	{
		for (int32 ConstraintIndex = 0; ConstraintIndex < NumConstraints(); ++ConstraintIndex)
		{
			ApplyPhase1Single(Dt, ConstraintIndex);
		}
	}

	void FPBDRigidSpringConstraints::AddBodies(const TArrayView<int32>& ConstraintIndices, FSolverBodyContainer& SolverBodyContainer)
	{
		for (int32 ConstraintIndex : ConstraintIndices)
		{
			AddBodies(ConstraintIndex, SolverBodyContainer);
		}
	}

	void FPBDRigidSpringConstraints::ScatterOutput(const TArrayView<int32>& ConstraintIndices, const FReal Dt)
	{
		for (int32 ConstraintIndex : ConstraintIndices)
		{
			ConstraintSolverBodies[ConstraintIndex] = 
			{ 
				nullptr, 
				nullptr 
			};
		}
	}

	void FPBDRigidSpringConstraints::ApplyPositionConstraints(const TArrayView<int32>& ConstraintIndices, const FReal Dt, const int32 It, const int32 NumIts)
	{
		for (int32 ConstraintIndex : ConstraintIndices)
		{
			ApplyPhase1Single(Dt, ConstraintIndex);
		}
	}

	void FPBDRigidSpringConstraints::AddBodies(const int32 ConstraintIndex, FSolverBodyContainer& SolverBodyContainer)
	{
		ConstraintSolverBodies[ConstraintIndex] =
		{
			SolverBodyContainer.FindOrAdd(Constraints[ConstraintIndex][0]),
			SolverBodyContainer.FindOrAdd(Constraints[ConstraintIndex][1])
		};
	}

	void FPBDRigidSpringConstraints::ApplyPhase1Single(const FReal Dt, int32 ConstraintIndex) const
	{
		check(ConstraintSolverBodies[ConstraintIndex][0] != nullptr);
		check(ConstraintSolverBodies[ConstraintIndex][1] != nullptr);
		FSolverBody& Body0 = *ConstraintSolverBodies[ConstraintIndex][0];
		FSolverBody& Body1 = *ConstraintSolverBodies[ConstraintIndex][1];
		const FVec3 BodyP0 = Body0.CorrectedP();
		const FRotation3 BodyQ0 = Body0.CorrectedQ();
		const FVec3 BodyP1 = Body1.CorrectedP();
		const FRotation3 BodyQ1 = Body1.CorrectedQ();

		if (!Body0.IsDynamic() && !Body1.IsDynamic())
		{
			return;
		}

		const FVec3 WorldSpaceX1 = BodyQ0.RotateVector(Distances[ConstraintIndex][0]) + BodyP0;
		const FVec3 WorldSpaceX2 = BodyQ1.RotateVector(Distances[ConstraintIndex][1]) + BodyP1;
		const FVec3 Delta = GetDelta(ConstraintIndex, WorldSpaceX1, WorldSpaceX2);

		if (Body0.IsDynamic())
		{
			const FVec3 Radius = WorldSpaceX1 - BodyP0;
			const FVec3 DX = Body0.InvM() * Delta;
			const FVec3 DR = Body0.InvI() * FVec3::CrossProduct(Radius, Delta);
			Body0.ApplyTransformDelta(DX, DR);
			Body0.UpdateRotationDependentState();
		}

		if (Body1.IsDynamic())
		{
			const FVec3 Radius = WorldSpaceX2 - BodyP1;
			const FVec3 DX = Body1.InvM() * -Delta;
			const FVec3 DR = Body1.InvI() * FVec3::CrossProduct(Radius, -Delta);
			Body1.ApplyTransformDelta(DX, DR);
			Body1.UpdateRotationDependentState();
		}
	}
}
