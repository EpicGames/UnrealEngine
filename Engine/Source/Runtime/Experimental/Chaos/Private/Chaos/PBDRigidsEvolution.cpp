// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/PBDRigidsEvolution.h"
#include "Chaos/Defines.h"
#include "Chaos/PBDRigidsSOAs.h"
#include "ChaosStats.h"
#include "Chaos/PBDRigidsEvolutionGBF.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/SpatialAccelerationCollection.h"
#include "Chaos/PhysicsMaterialUtilities.h"
#include "Tasks/Task.h"

CSV_DECLARE_CATEGORY_EXTERN(ChaosPhysicsTimers);

int32 ChaosRigidsEvolutionApplyAllowEarlyOutCVar = 1;
FAutoConsoleVariableRef CVarChaosRigidsEvolutionApplyAllowEarlyOut(TEXT("p.ChaosRigidsEvolutionApplyAllowEarlyOut"), ChaosRigidsEvolutionApplyAllowEarlyOutCVar, TEXT("Allow Chaos Rigids Evolution apply iterations to early out when resolved.[def:1]"));

int32 ChaosRigidsEvolutionApplyPushoutAllowEarlyOutCVar = 1;
FAutoConsoleVariableRef CVarChaosRigidsEvolutionApplyPushoutAllowEarlyOut(TEXT("p.ChaosRigidsEvolutionApplyPushoutAllowEarlyOut"), ChaosRigidsEvolutionApplyPushoutAllowEarlyOutCVar, TEXT("Allow Chaos Rigids Evolution apply-pushout iterations to early out when resolved.[def:1]"));

int32 ChaosNumPushOutIterationsOverride = -1;
FAutoConsoleVariableRef CVarChaosNumPushOutIterationsOverride(TEXT("p.ChaosNumPushOutIterationsOverride"), ChaosNumPushOutIterationsOverride, TEXT("Override for num push out iterations if >= 0 [def:-1]"));

int32 ChaosNumContactIterationsOverride = -1;
FAutoConsoleVariableRef CVarChaosNumContactIterationsOverride(TEXT("p.ChaosNumContactIterationsOverride"), ChaosNumContactIterationsOverride, TEXT("Override for num contact iterations if >= 0. [def:-1]"));

namespace Chaos
{
	namespace CVars
	{
		extern bool bChaos_Solver_TestMode_Enabled;
		extern int32 Chaos_Solver_TestMode_Step;
	}

	CHAOS_API int32 FixBadAccelerationStructureRemoval = 1;
	FAutoConsoleVariableRef CVarFixBadAccelerationStructureRemoval(TEXT("p.FixBadAccelerationStructureRemoval"), FixBadAccelerationStructureRemoval, TEXT(""));

	CHAOS_API int32 AccelerationStructureIsolateQueryOnlyObjects = 0;
	FAutoConsoleVariableRef CVarAccelerationStructureIsolateQueryOnlyObjects(TEXT("p.Chaos.AccelerationStructureIsolateQueryOnlyObjects"), AccelerationStructureIsolateQueryOnlyObjects, TEXT("Set to 1: QueryOnly Objects will not be moved to acceleration structures on the Physics Thread"));

	CHAOS_API int32 AccelerationStructureSplitStaticAndDynamic = 1;
	FAutoConsoleVariableRef CVarAccelerationStructureSplitStaticAndDynamic(TEXT("p.Chaos.AccelerationStructureSplitStaticDynamic"), AccelerationStructureSplitStaticAndDynamic, TEXT("Set to 1: Sort Dynamic and Static bodies into separate acceleration structures, any other value will disable the feature"));

	CHAOS_API int32 AccelerationStructureUseDynamicTree = 1;
	FAutoConsoleVariableRef CVarAccelerationStructureUseDynamicTree(TEXT("p.Chaos.AccelerationStructureUseDynamicTree"), AccelerationStructureUseDynamicTree, TEXT("Use a dynamic BVH tree structure for dynamic objects"));

	CHAOS_API int32 AccelerationStructureUseDirtyTreeInsteadOfGrid = 1;
	FAutoConsoleVariableRef CVarAccelerationStructureUseDirtyTreeInsteadOfGrid(TEXT("p.Chaos.AccelerationStructureUseDirtyTreeInsteadOfGrid"), AccelerationStructureUseDirtyTreeInsteadOfGrid, TEXT("Use a dynamic tree structure for dirty elements instead of a 2D grid"));

	/** Console variable to enable the caching of the overlapping leaves if the dynamic tree is enable */
	CHAOS_API int32 GAccelerationStructureCacheOverlappingLeaves = 1;
	FAutoConsoleVariableRef CVarAccelerationStructureCacheOverlappingLeaves(TEXT("p.Chaos.AccelerationStructureCacheOverlappingLeaves"), GAccelerationStructureCacheOverlappingLeaves, TEXT("Set to 1: Cache the overlapping leaves for faster overlap query, any other value will disable the feature"));

	CHAOS_API int32 AccelerationStructureTimeSlicingMaxQueueSizeBeforeForce = 1000;
	FAutoConsoleVariableRef CVarAccelerationStructureTimeSlicingMaxQueueSizeBeforeForce(TEXT("p.Chaos.AccelerationStructureTimeSlicingMaxQueueSizeBeforeForce"), AccelerationStructureTimeSlicingMaxQueueSizeBeforeForce, TEXT("If the update queue reaches this limit, time slicing will be disabled, and the acceleration structure will be built at once"));

	CHAOS_API int32 AccelerationStructureTimeSlicingMaxBytesCopy = 100000;
	FAutoConsoleVariableRef CVarAccelerationStructureTimeSlicingMaxBytesCopy(TEXT("p.Chaos.AccelerationStructureTimeSlicingMaxBytesCopy"), AccelerationStructureTimeSlicingMaxBytesCopy, TEXT("The Maximum number of bytes to copy to the external acceleration structure during Copy Time Slicing"));

	bool bPruningInSingleWorkerPhysics = false;
	FAutoConsoleVariableRef CVarPruningInSingleWorkerPhysics(TEXT("p.Chaos.PruningInSingleWorkerPhysics"), bPruningInSingleWorkerPhysics, TEXT("Enable Pruning to update acceleration structure in Single Worker mode.\
																Pruning the acceleration structure is an optimization technique that speed up the spatial acceleration update which is a normally a single threaded process by using several threads to prune data to avoid some unecessary update. \
																In single worker physics it would be better to disable it."));

	CHAOS_API FBroadPhaseConfig BroadPhaseConfig;

	FAutoConsoleVariableRef CVarBroadphaseIsTree(TEXT("p.BroadphaseType"), BroadPhaseConfig.BroadphaseType, TEXT(""));
	FAutoConsoleVariableRef CVarBoundingVolumeNumCells(TEXT("p.BoundingVolumeNumCells"), BroadPhaseConfig.BVNumCells, TEXT(""));
	FAutoConsoleVariableRef CVarMaxChildrenInLeaf(TEXT("p.MaxChildrenInLeaf"), BroadPhaseConfig.MaxChildrenInLeaf, TEXT(""));
	FAutoConsoleVariableRef CVarMaxTreeDepth(TEXT("p.MaxTreeDepth"), BroadPhaseConfig.MaxTreeDepth, TEXT(""));
	FAutoConsoleVariableRef CVarAABBMaxChildrenInLeaf(TEXT("p.AABBMaxChildrenInLeaf"), BroadPhaseConfig.AABBMaxChildrenInLeaf, TEXT(""));
	FAutoConsoleVariableRef CVarAABBMaxTreeDepth(TEXT("p.AABBMaxTreeDepth"), BroadPhaseConfig.AABBMaxTreeDepth, TEXT(""));
	FAutoConsoleVariableRef CVarMaxPayloadSize(TEXT("p.MaxPayloadSize"), BroadPhaseConfig.MaxPayloadSize, TEXT(""));
	FAutoConsoleVariableRef CVarIterationsPerTimeSlice(TEXT("p.IterationsPerTimeSlice"), BroadPhaseConfig.IterationsPerTimeSlice, TEXT(""));

	struct FDefaultCollectionFactory : public ISpatialAccelerationCollectionFactory
	{
		using BVType = TBoundingVolume<FAccelerationStructureHandle>;
		using AABBTreeType = TAABBTree<FAccelerationStructureHandle, TAABBTreeLeafArray<FAccelerationStructureHandle>>;
		using AABBDynamicTreeType = TAABBTree<FAccelerationStructureHandle, TAABBTreeLeafArray<FAccelerationStructureHandle>>;
		using AABBTreeOfGridsType = TAABBTree<FAccelerationStructureHandle, TBoundingVolume<FAccelerationStructureHandle>>;


		static bool IsDynamicTree(FSpatialAccelerationIdx SpatialAccelerationIdx)
		{
			constexpr uint16 DynamicAsUInt16 = static_cast<std::underlying_type_t<ESpatialAccelerationCollectionBucketInnerIndex>>(ESpatialAccelerationCollectionBucketInnerIndex::Dynamic);
			constexpr uint16 DynamicQueryOnlyAsUInt16 = static_cast<std::underlying_type_t<ESpatialAccelerationCollectionBucketInnerIndex>>(ESpatialAccelerationCollectionBucketInnerIndex::DynamicQueryOnly);			
			return AccelerationStructureUseDynamicTree && (SpatialAccelerationIdx.InnerIdx == DynamicAsUInt16 || SpatialAccelerationIdx.InnerIdx == DynamicQueryOnlyAsUInt16);
		}

		TUniquePtr<ISpatialAccelerationCollection<FAccelerationStructureHandle, FReal, 3>> CreateEmptyCollection() override
		{
			TConstParticleView<FSpatialAccelerationCache> Empty;

			const uint16 NumBuckets = BroadPhaseConfig.BroadphaseType >= FBroadPhaseConfig::TreeAndGrid ? 2 : 1;
			auto Collection = new TSpatialAccelerationCollection<AABBTreeType, BVType, AABBTreeOfGridsType>();

			for (uint16 BucketIdx = 0; BucketIdx < NumBuckets; ++BucketIdx)
			{
				// For static bodies
				constexpr uint16 DefaultAsUInt16 = static_cast<std::underlying_type_t<ESpatialAccelerationCollectionBucketInnerIndex>>(ESpatialAccelerationCollectionBucketInnerIndex::Default);
				Collection->AddSubstructure(CreateAccelerationPerBucket_Threaded(Empty, BucketIdx, true, false, false), BucketIdx, DefaultAsUInt16);

				// Always create this if QueryOnlyObjects are isolated to ensure all consecutive indices are created (within buckets)
				if (AccelerationStructureSplitStaticAndDynamic == 1 || AccelerationStructureIsolateQueryOnlyObjects == 1) 
				{
					constexpr uint16 DynamicAsUInt16 = static_cast<std::underlying_type_t<ESpatialAccelerationCollectionBucketInnerIndex>>(ESpatialAccelerationCollectionBucketInnerIndex::Dynamic);
					// Non static bodies
					Collection->AddSubstructure(CreateAccelerationPerBucket_Threaded(Empty, BucketIdx, true, IsDynamicTree(FSpatialAccelerationIdx{BucketIdx, DynamicAsUInt16 }), true), BucketIdx, DynamicAsUInt16);
				}
			}

			if (AccelerationStructureIsolateQueryOnlyObjects && NumBuckets == 1)
			{
				const auto DefaultQueryOnlyAsUInt16 = static_cast<std::underlying_type_t<ESpatialAccelerationCollectionBucketInnerIndex>>(ESpatialAccelerationCollectionBucketInnerIndex::DefaultQueryOnly);
				constexpr uint16 BucketIdx = 0;
				Collection->AddSubstructure(CreateAccelerationPerBucket_Threaded(Empty, BucketIdx, true, false, false), BucketIdx, DefaultQueryOnlyAsUInt16);
				if (AccelerationStructureSplitStaticAndDynamic == 1)
				{
					constexpr uint16 DynamicQueryOnlyAsUInt16 = static_cast<std::underlying_type_t<ESpatialAccelerationCollectionBucketInnerIndex>>(ESpatialAccelerationCollectionBucketInnerIndex::DynamicQueryOnly);

					// Non static bodies
					Collection->AddSubstructure(CreateAccelerationPerBucket_Threaded(Empty, BucketIdx, true, IsDynamicTree(FSpatialAccelerationIdx{ BucketIdx,  DynamicQueryOnlyAsUInt16 }), false), BucketIdx, DynamicQueryOnlyAsUInt16);
				}
			}

			return TUniquePtr<ISpatialAccelerationCollection<FAccelerationStructureHandle, FReal, 3>>(Collection);
		}

		virtual uint8 GetActiveBucketsMask() const
		{
			return BroadPhaseConfig.BroadphaseType >= FBroadPhaseConfig::TreeAndGrid ? 3 : 1;
		}

		virtual bool IsBucketTimeSliced(uint16 BucketIdx) const
		{
			// TODO: Unduplicate switch statement here with CreateAccelerationPerBucket_Threaded and refactor so that bucket index mapping is better.
			switch (BucketIdx)
			{
			case 0:
			{
				if (BroadPhaseConfig.BroadphaseType == FBroadPhaseConfig::Grid)
				{
					// BVType
					return false;
				}
				else if (BroadPhaseConfig.BroadphaseType == FBroadPhaseConfig::Tree || BroadPhaseConfig.BroadphaseType == FBroadPhaseConfig::TreeAndGrid)
				{
					// AABBTreeType
					return true;
				}
				else if (BroadPhaseConfig.BroadphaseType == FBroadPhaseConfig::TreeOfGridAndGrid || BroadPhaseConfig.BroadphaseType == FBroadPhaseConfig::TreeOfGrid)
				{
					// AABBTreeOfGridsType
					return true;
				}
			}
			case 1:
			{
				// BVType
				ensure(BroadPhaseConfig.BroadphaseType == FBroadPhaseConfig::TreeAndGrid || BroadPhaseConfig.BroadphaseType == FBroadPhaseConfig::TreeOfGridAndGrid);
				return false;
			}
			default:
			{
				check(false);
				return false;
			}
			}
		}

		virtual TUniquePtr<ISpatialAcceleration<FAccelerationStructureHandle, FReal, 3>> CreateAccelerationPerBucket_Threaded(const TConstParticleView<FSpatialAccelerationCache>& Particles, uint16 BucketIdx, bool ForceFullBuild, bool bDynamicTree, bool bBuildOverlapCache) override
		{
			// TODO: Unduplicate switch statement here with IsBucketTimeSliced and refactor so that bucket index mapping is better.
			switch (BucketIdx)
			{
			case 0:
			{
				if (BroadPhaseConfig.BroadphaseType == FBroadPhaseConfig::Grid)
				{
					return MakeUnique<BVType>(Particles, false, static_cast<FReal>(0), BroadPhaseConfig.BVNumCells, BroadPhaseConfig.MaxPayloadSize);
				}
				else if (BroadPhaseConfig.BroadphaseType == FBroadPhaseConfig::Tree || BroadPhaseConfig.BroadphaseType == FBroadPhaseConfig::TreeAndGrid)
				{
					bool bUseDirtyTree = false;
					const int MaxNumToProcess = ForceFullBuild ? 0 : BroadPhaseConfig.IterationsPerTimeSlice;
					if (bDynamicTree)
					{
						return MakeUnique<AABBDynamicTreeType>(Particles, BroadPhaseConfig.MaxChildrenInLeaf, BroadPhaseConfig.MaxTreeDepth, BroadPhaseConfig.MaxPayloadSize, MaxNumToProcess, bDynamicTree, bUseDirtyTree, bBuildOverlapCache);
					}
					bUseDirtyTree = AccelerationStructureUseDirtyTreeInsteadOfGrid == 1;
					return MakeUnique<AABBTreeType>(Particles, BroadPhaseConfig.MaxChildrenInLeaf, BroadPhaseConfig.MaxTreeDepth, BroadPhaseConfig.MaxPayloadSize, MaxNumToProcess, bDynamicTree, bUseDirtyTree, bBuildOverlapCache);
				}
				else if (BroadPhaseConfig.BroadphaseType == FBroadPhaseConfig::TreeOfGridAndGrid || BroadPhaseConfig.BroadphaseType == FBroadPhaseConfig::TreeOfGrid)
				{
					return MakeUnique<AABBTreeOfGridsType>(Particles, BroadPhaseConfig.AABBMaxChildrenInLeaf, BroadPhaseConfig.AABBMaxTreeDepth, BroadPhaseConfig.MaxPayloadSize);
				}
			}
			case 1:
			{
				ensure(BroadPhaseConfig.BroadphaseType == FBroadPhaseConfig::TreeAndGrid || BroadPhaseConfig.BroadphaseType == FBroadPhaseConfig::TreeOfGridAndGrid);
				return MakeUnique<BVType>(Particles, false, static_cast<FReal>(0), BroadPhaseConfig.BVNumCells, BroadPhaseConfig.MaxPayloadSize);
			}
			default:
			{
				check(false);
				return nullptr;
			}
			}
		}

		virtual void Serialize(TUniquePtr<ISpatialAccelerationCollection<FAccelerationStructureHandle, FReal, 3>>& Ptr, FChaosArchive& Ar) override
		{
			if (Ar.IsLoading())
			{
				Ptr = CreateEmptyCollection();
				Ptr->Serialize(Ar);
			}
			else
			{
				Ptr->Serialize(Ar);
			}
		}
	};

	FPBDRigidsEvolutionBase::FPBDRigidsEvolutionBase(FPBDRigidsSOAs& InParticles, THandleArray<FChaosPhysicsMaterial>& InSolverPhysicsMaterials, bool InIsSingleThreaded)
	    : IslandManager(InParticles)
		, IslandGroupManager(IslandManager)
		, Particles(InParticles)
		, SolverPhysicsMaterials(InSolverPhysicsMaterials)
		, InternalAcceleration(nullptr)
		, AsyncInternalAcceleration(nullptr)
		, AsyncExternalAcceleration(nullptr)
		, bIsSingleThreaded(InIsSingleThreaded)
		, bCanStartAsyncTasks(true)
		, LatestExternalTimestampConsumed_Internal(-1)
		, TaskDispatcher(Particles, InternalAccelerationQueue)
		, bAccelerationStructureTaskStarted(nullptr)
		, bAccelerationStructureTaskSignalKill(nullptr)
		, SpatialCollectionFactory(new FDefaultCollectionFactory())
	{
		Particles.GetParticleHandles().AddArray(&PhysicsMaterials);
		Particles.GetParticleHandles().AddArray(&PerParticlePhysicsMaterials);
		Particles.GetParticleHandles().AddArray(&Collided);

		for (auto& Particle : InParticles.GetNonDisabledView())
		{
			DirtyParticle(Particle);
		}

		ComputeIntermediateSpatialAcceleration();
	}

	FPBDRigidsEvolutionBase::~FPBDRigidsEvolutionBase()
	{
		Particles.GetParticleHandles().RemoveArray(&PhysicsMaterials);
		Particles.GetParticleHandles().RemoveArray(&PerParticlePhysicsMaterials);
		Particles.GetParticleHandles().RemoveArray(&Collided);
		WaitOnAccelerationStructure();
	}


	DECLARE_CYCLE_STAT(TEXT("CacheAccelerationBounds"), STAT_CacheAccelerationBounds, STATGROUP_Chaos);
	DECLARE_CYCLE_STAT(TEXT("ComputeIntermediateSpatialAcceleration"), STAT_ComputeIntermediateSpatialAcceleration, STATGROUP_Chaos);
	DECLARE_CYCLE_STAT(TEXT("CopyAccelerationStructure"), STAT_CopyAccelerationStructure, STATGROUP_Chaos);
	DECLARE_CYCLE_STAT(TEXT("SwapAccelerationStructures"), STAT_SwapAccelerationStructures, STATGROUP_Chaos);
	DECLARE_CYCLE_STAT(TEXT("AccelerationStructureTimeSlice"), STAT_AccelerationStructureTimeSlice, STATGROUP_Chaos);
	DECLARE_CYCLE_STAT(TEXT("AccelerationStructureTimeSliceCopy"), STAT_AccelerationStructureTimeSliceCopy, STATGROUP_Chaos);
	DECLARE_CYCLE_STAT(TEXT("CreateInitialAccelerationStructure"), STAT_CreateInitialAccelerationStructure, STATGROUP_Chaos);
	DECLARE_CYCLE_STAT(TEXT("CreateNonSlicedStructures"), STAT_CreateNonSlicedStructures, STATGROUP_Chaos);

	FPBDRigidsEvolutionBase::FChaosAccelerationStructureTask::FChaosAccelerationStructureTask(
		ISpatialAccelerationCollectionFactory& InSpatialCollectionFactory
		, const TMap<FSpatialAccelerationIdx, TUniquePtr<FSpatialAccelerationCache>>& InSpatialAccelerationCache
		, FAccelerationStructure* InInternalAccelerationStructure
		, FAccelerationStructure* InExternalAccelerationStructure
		, bool InForceFullBuild
		, bool InIsSingleThreaded
		, bool InNeedsReset
		, std::atomic<bool>** bOutStarted
		, std::atomic<bool>** bOutKillTask)
		: SpatialCollectionFactory(InSpatialCollectionFactory)
		, SpatialAccelerationCache(InSpatialAccelerationCache)
		, InternalStructure(InInternalAccelerationStructure)
		, ExternalStructure(InExternalAccelerationStructure)
		, IsForceFullBuild(InForceFullBuild)
		, bIsSingleThreaded(InIsSingleThreaded)
		, bNeedsReset(InNeedsReset)
		, bStarted(false)
		, bKillTask(false)
	{
		*bOutStarted = &bStarted;
		*bOutKillTask = &bKillTask;
	}

	TStatId FPBDRigidsEvolutionBase::FChaosAccelerationStructureTask::GetStatId()
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FChaosAccelerationStructureTask, STATGROUP_Chaos);
	}

	ENamedThreads::Type FPBDRigidsEvolutionBase::FChaosAccelerationStructureTask::GetDesiredThread()
	{
#if WITH_EDITOR
		// Heavy async compilation could fill the background threads in Editor preventing us from making
		// progress which would cause game-thread stalls. Schedule as normal for Editor to avoid this.
		return ENamedThreads::AnyNormalThreadNormalTask;
#else
		return ENamedThreads::AnyBackgroundThreadNormalTask;
#endif
	}

	ESubsequentsMode::Type FPBDRigidsEvolutionBase::FChaosAccelerationStructureTask::GetSubsequentsMode()
	{
		return ESubsequentsMode::TrackSubsequents;
	}

	TUniquePtr<ISpatialAccelerationCollection<FAccelerationStructureHandle, FReal, 3>> CreateNewSpatialStructureFromSubStructure(TUniquePtr<ISpatialAcceleration<FAccelerationStructureHandle, FReal, 3>>&& Substructure)
	{
		using BVType = TBoundingVolume<FAccelerationStructureHandle>;
		using AABBType = TAABBTree<FAccelerationStructureHandle, TAABBTreeLeafArray<FAccelerationStructureHandle>>;

		if (Substructure->template As<BVType>())
		{
			auto Collection = MakeUnique<TSpatialAccelerationCollection<BVType>>();
			Collection->AddSubstructure(MoveTemp(Substructure), 0, 0);
			return Collection;
		}
		else if (Substructure->template As<AABBType>())
		{
			auto Collection = MakeUnique<TSpatialAccelerationCollection<AABBType>>();
			Collection->AddSubstructure(MoveTemp(Substructure), 0, 0);
			return Collection;
		}
		else
		{
			using AccelType = TAABBTree<FAccelerationStructureHandle, TBoundingVolume<FAccelerationStructureHandle>>;
			auto Collection = MakeUnique<TSpatialAccelerationCollection<AccelType>>();
			Collection->AddSubstructure(MoveTemp(Substructure), 0, 0);
			return Collection;
		}
	}

	void FPBDRigidsEvolutionBase::FChaosAccelerationStructureTask::UpdateStructure(FAccelerationStructure* AccelerationStructure, FAccelerationStructure* AccelerationStructureCopy)
	{
		LLM_SCOPE(ELLMTag::ChaosAcceleration);

		// This function (running in FChaosAccelerationStructureTask) can be in one of two modes
		// 1) Create new time sliced acceleration structures and run the first slice  (CreatingNewTimeSlicedStructures)
		// or
		// 2) Progress time slices (!CreatingNewTimeSlicedStructures)
		// In either case, if time slicing is done create the non time sliced acceleration structures


		auto GetSubStructureCopy = [AccelerationStructureCopy](FSpatialAccelerationIdx Index)
		{
			ISpatialAcceleration<FAccelerationStructureHandle, FReal, 3>* Result = nullptr;
			if (AccelerationStructureCopy)
			{
				Result = AccelerationStructureCopy->GetSubstructure(Index);
				if (!Result)
				{
					ensure(false);
				}
				// We should not copy dynamic trees here
				else if (Result->IsTreeDynamic())
				{
					Result = nullptr;
				}
			}
			return Result;
		};

		uint8 ActiveBucketsMask = SpatialCollectionFactory.GetActiveBucketsMask();
		TArray<TSOAView<FSpatialAccelerationCache>> ViewsPerBucket[FSpatialAccelerationIdx::MaxBuckets];
		TArray<FSpatialAccelerationIdx> TimeSlicedBucketsToCreate;
		TArray<FSpatialAccelerationIdx> NonTimeSlicedBucketsToCreate;

		bool IsTimeSlicingProgressing = false;
		bool CreatingNewTimeSlicedStructures = bNeedsReset;

		if (CreatingNewTimeSlicedStructures && AccelerationStructureCopy)
		{
			// Cannot assume all substructures that exist are in cache!!
			// Make sure AccelerationStructureCopy is completely reset and does not have dirty substructures that will be ignored below.
			AccelerationStructureCopy->Reset();
		}

		for(const auto& Itr : SpatialAccelerationCache)
		{
			const FSpatialAccelerationIdx SpatialIdx = Itr.Key;
			const FSpatialAccelerationCache& Cache = *Itr.Value;
			const uint8 BucketIdx = static_cast<uint8>((1 << SpatialIdx.Bucket) & ActiveBucketsMask ? SpatialIdx.Bucket : 0);
			const uint16 BucketInnerIdx = SpatialIdx.InnerIdx;
			ISpatialAcceleration<FAccelerationStructureHandle, FReal, 3>* SubStructure = AccelerationStructure->GetSubstructure(SpatialIdx);
			// Get the sub structure we need to copy our results to
			ISpatialAcceleration<FAccelerationStructureHandle, FReal, 3>* AccelerationSUBStructureCopy = GetSubStructureCopy(SpatialIdx);

			if (!SubStructure)
			{
				ensure(false);
				continue;
			}
			
			if (CreatingNewTimeSlicedStructures && !SubStructure->ShouldRebuild() && AccelerationStructureSplitStaticAndDynamic == 1)
			{
				ensure(SubStructure->IsAsyncTimeSlicingComplete());

				if (AccelerationSUBStructureCopy)
				{
					AccelerationSUBStructureCopy->PrepareCopyTimeSliced(*SubStructure);
					ensure(!AccelerationSUBStructureCopy->IsAsyncTimeSlicingComplete());
					AccelerationSUBStructureCopy->ProgressCopyTimeSliced(*SubStructure, IsForceFullBuild ? -1 : AccelerationStructureTimeSlicingMaxBytesCopy);
					if (!AccelerationSUBStructureCopy->IsAsyncTimeSlicingComplete())
					{
						IsTimeSlicingProgressing = true;
					}
				}

				// This structure has not changed and therefore does not need to rebuild 
				continue;
			}

			if (CreatingNewTimeSlicedStructures)
			{
				SubStructure->Reset();
			}

			if(!SubStructure->IsAsyncTimeSlicingComplete())
			{
				SCOPE_CYCLE_COUNTER(STAT_AccelerationStructureTimeSlice);
				
				SubStructure->ProgressAsyncTimeSlicing(IsForceFullBuild);

				// is it still progressing or now complete
				if(!SubStructure->IsAsyncTimeSlicingComplete())
				{
					IsTimeSlicingProgressing = true;
				}
				else
				{
					if (AccelerationSUBStructureCopy)
					{
						AccelerationSUBStructureCopy->PrepareCopyTimeSliced(*SubStructure);
						IsTimeSlicingProgressing |= !AccelerationSUBStructureCopy->IsAsyncTimeSlicingComplete();
					}
				}
			}
			else if (AccelerationSUBStructureCopy && !AccelerationSUBStructureCopy->IsAsyncTimeSlicingComplete())
			{
				SCOPE_CYCLE_COUNTER(STAT_AccelerationStructureTimeSliceCopy);
				AccelerationSUBStructureCopy->ProgressCopyTimeSliced(*SubStructure, IsForceFullBuild ? -1 : AccelerationStructureTimeSlicingMaxBytesCopy);
				if (!AccelerationSUBStructureCopy->IsAsyncTimeSlicingComplete())
				{
					IsTimeSlicingProgressing = true;
				}
			}
			else
			{
				// Queue for potential Creation
				ViewsPerBucket[BucketIdx].SetNum(FMath::Max(BucketInnerIdx + 1, ViewsPerBucket[BucketIdx].Num()));
				ViewsPerBucket[BucketIdx][BucketInnerIdx] = const_cast<FSpatialAccelerationCache*>(&Cache);
				
				if(SpatialCollectionFactory.IsBucketTimeSliced(BucketIdx))
				{
					TimeSlicedBucketsToCreate.Add(SpatialIdx);
				} else
				{
					NonTimeSlicedBucketsToCreate.Add(SpatialIdx);
				}
			}
		}

		//todo: creation can go wide, insertion to collection cannot
		if (CreatingNewTimeSlicedStructures)
		{
			for (FSpatialAccelerationIdx SpatialAccIdx : TimeSlicedBucketsToCreate)
			{
				SCOPE_CYCLE_COUNTER(STAT_CreateInitialAccelerationStructure);

				auto ParticleView = MakeConstParticleView(MoveTemp(ViewsPerBucket[SpatialAccIdx.Bucket][SpatialAccIdx.InnerIdx]));
				// Create and run the first slice here
				auto NewStruct = SpatialCollectionFactory.CreateAccelerationPerBucket_Threaded(ParticleView, SpatialAccIdx.Bucket, IsForceFullBuild, false, false);
				{
					NewStruct->ClearShouldRebuild();  // Mark as pristine
				}

				ISpatialAcceleration<FAccelerationStructureHandle, FReal, 3>* AccelerationSUBStructureCopy = GetSubStructureCopy(SpatialAccIdx);

				if (AccelerationSUBStructureCopy && NewStruct->IsAsyncTimeSlicingComplete())
				{
					AccelerationSUBStructureCopy->PrepareCopyTimeSliced(*NewStruct);
					AccelerationSUBStructureCopy->ProgressCopyTimeSliced(*NewStruct, IsForceFullBuild ? -1 : AccelerationStructureTimeSlicingMaxBytesCopy);
					IsTimeSlicingProgressing |= !AccelerationSUBStructureCopy->IsAsyncTimeSlicingComplete();
				}

				//If new structure is not done mark time slicing in progress
				IsTimeSlicingProgressing |= !NewStruct->IsAsyncTimeSlicingComplete();

				if (AccelerationStructure->IsBucketActive(SpatialAccIdx.Bucket))
				{
					AccelerationStructure->RemoveSubstructure(SpatialAccIdx);
				}
				AccelerationStructure->AddSubstructure(MoveTemp(NewStruct), SpatialAccIdx.Bucket, SpatialAccIdx.InnerIdx);
			}
		}
		// If it's not progressing then it is finished so we can perform the final copy if required
		AccelerationStructure->SetAllAsyncTasksComplete(!IsTimeSlicingProgressing);

		// Create and Build the non time sliced structures when all time sliced structures are complete
		if(!IsTimeSlicingProgressing)
		{
			//todo: creation can go wide, insertion to collection cannot
			for(FSpatialAccelerationIdx SpatialAccIdx : NonTimeSlicedBucketsToCreate)
			{
				if(ViewsPerBucket[SpatialAccIdx.Bucket].Num() > SpatialAccIdx.InnerIdx)
				{
					SCOPE_CYCLE_COUNTER(STAT_CreateNonSlicedStructures);

					auto ParticleView = MakeConstParticleView(MoveTemp(ViewsPerBucket[SpatialAccIdx.Bucket][SpatialAccIdx.InnerIdx]));
					auto NewStruct = SpatialCollectionFactory.CreateAccelerationPerBucket_Threaded(ParticleView,SpatialAccIdx.Bucket,IsForceFullBuild, false, false);

					ISpatialAcceleration<FAccelerationStructureHandle, FReal, 3>* AccelerationSUBStructureCopy = GetSubStructureCopy(SpatialAccIdx);
					if (AccelerationSUBStructureCopy)
					{
						AccelerationSUBStructureCopy->DeepAssign(*NewStruct); // Normal copy
					}
					if (AccelerationStructure->IsBucketActive(SpatialAccIdx.Bucket))
					{
						AccelerationStructure->RemoveSubstructure(SpatialAccIdx);
					}
					AccelerationStructure->AddSubstructure(MoveTemp(NewStruct),SpatialAccIdx.Bucket, SpatialAccIdx.InnerIdx);
				}
			}
		}
	}

	void FPBDRigidsEvolutionBase::FChaosAccelerationStructureTask::DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		bStarted = true;
		if (bKillTask) // We can kill the task before it has really started
		{
			return;
		}		

		LLM_SCOPE(ELLMTag::ChaosAcceleration);

#if !WITH_EDITOR
		//CSV_SCOPED_TIMING_STAT(ChaosPhysicsTimers, AccelerationStructBuildTask);
#endif
		if (AccelerationStructureSplitStaticAndDynamic == 1)
		{
			UpdateStructure(InternalStructure, ExternalStructure);
		}
		else
		{
			//Rebuild both structures. TODO: probably faster to time slice the copy instead of doing two time sliced builds (This happens with AccelerationStructureSplitStaticAndDynamic)
			UpdateStructure(InternalStructure);
			UpdateStructure(ExternalStructure);
		}
	}

	void FPBDRigidsEvolutionBase::ApplyParticlePendingData(const FPendingSpatialData& SpatialData, FAccelerationStructure& AccelerationStructure, bool bUpdateCache, bool bUpdateDynamicTrees)
	{
		if (SpatialData.Operation ==  EPendingSpatialDataOperation::Delete)
		{
			if (bUpdateDynamicTrees || !FDefaultCollectionFactory::IsDynamicTree(SpatialData.SpatialIdx))
			{
				const bool bExisted = AccelerationStructure.RemoveElementFrom(SpatialData.AccelerationHandle, SpatialData.SpatialIdx);
				//ensure(bExisted);
			}

			if (bUpdateCache)
			{
				if (uint32* InnerIdxPtr = ParticleToCacheInnerIdx.Find(SpatialData.UniqueIdx()))
				{
					FSpatialAccelerationCache& Cache = *SpatialAccelerationCache.FindChecked(SpatialData.SpatialIdx);	//can't delete from cache that doesn't exist
					const uint32 CacheInnerIdx = *InnerIdxPtr;
					if (CacheInnerIdx + 1 < Cache.Size())	//will get swapped with last element, so update it
					{
						const FUniqueIdx LastParticleInCacheUniqueIdx = Cache.Payload(Cache.Size() - 1).UniqueIdx();
						ParticleToCacheInnerIdx.FindChecked(LastParticleInCacheUniqueIdx) = CacheInnerIdx;
					}

					Cache.DestroyElement(CacheInnerIdx);
					ParticleToCacheInnerIdx.Remove(SpatialData.UniqueIdx());
				}
			}
		}
		else
		{
			FGeometryParticleHandle* UpdateParticle = SpatialData.AccelerationHandle.GetGeometryParticleHandle_PhysicsThread();

			if (bUpdateDynamicTrees || !FDefaultCollectionFactory::IsDynamicTree(SpatialData.SpatialIdx))
			{
				//CSV_SCOPED_TIMING_STAT(ChaosPhysicsTimers, AABBTreeUpdate)
				const bool bExisted = AccelerationStructure.UpdateElementIn(UpdateParticle, UpdateParticle->WorldSpaceInflatedBounds(), UpdateParticle->HasBounds(), SpatialData.SpatialIdx);
				if (SpatialData.Operation == EPendingSpatialDataOperation::Add)
				{
					//ensure(bExisted == false); // TODO use this for ADD/UPDATE Audit
				}
				else
				{
					//ensure(bExisted == true);	// TODO use this for ADD/UPDATE Audit				
				}
			}

			if (bUpdateCache)
			{
				TUniquePtr<FSpatialAccelerationCache>* CachePtrPtr = SpatialAccelerationCache.Find(SpatialData.SpatialIdx);
				if (CachePtrPtr == nullptr)
				{
					CachePtrPtr = &SpatialAccelerationCache.Add(SpatialData.SpatialIdx, TUniquePtr<FSpatialAccelerationCache>(new FSpatialAccelerationCache));
				}

				FSpatialAccelerationCache& Cache = **CachePtrPtr;

				//make sure in mapping
				uint32 CacheInnerIdx;
				if (uint32* CacheInnerIdxPtr = ParticleToCacheInnerIdx.Find(SpatialData.UniqueIdx()))
				{
					CacheInnerIdx = *CacheInnerIdxPtr;
				}
				else
				{
					CacheInnerIdx = Cache.Size();
					Cache.AddElements(1);
					ParticleToCacheInnerIdx.Add(SpatialData.UniqueIdx(), CacheInnerIdx);
				}

				//update cache entry
				Cache.HasBounds(CacheInnerIdx) = UpdateParticle->HasBounds();
				Cache.Bounds(CacheInnerIdx) = UpdateParticle->WorldSpaceInflatedBounds();
				Cache.Payload(CacheInnerIdx) = SpatialData.AccelerationHandle;
			}
		}
	}

	void FPBDRigidsEvolutionBase::PruneInternalPendingData(FAccelerationStructure& AccelerationStructure)
	{
		// Remove all particles which are not updating their leaf in the acceleration structure.
		// The bounding volume associated to the particle will be updated in the acceleration structure.
		QUICK_SCOPE_CYCLE_COUNTER(STAT_PruneInternalPendingData);


		auto HasToBeUpdated = [&AccelerationStructure](const FPendingSpatialData& PendingData) -> bool
		{
			FGeometryParticleHandle* UpdateParticle = PendingData.AccelerationHandle.GetGeometryParticleHandle_PhysicsThread();
			return (PendingData.Operation == EPendingSpatialDataOperation::Delete ||
				(UpdateParticle->HasBounds() && AccelerationStructure.NeedUpdateElementIn(UpdateParticle, UpdateParticle->WorldSpaceInflatedBounds(), PendingData.SpatialIdx)));

		};

		if (Chaos::ShouldExecuteTasks(Particles.GetActiveDynamicMovingKinematicParticlesView().Num()))
		{
			TaskDispatcher.PruneInternalPendingData(HasToBeUpdated);
		}
		else
		{
			InternalAccelerationQueue.CleanUpDuplicated();
			if (bPruningInSingleWorkerPhysics)
			{
				for (TArray<FPendingSpatialData>& PendingDataArray : InternalAccelerationQueue.PendingDataArrays)
				{
					TArray<FPendingSpatialData> NewPendingData;
					NewPendingData.Reserve(PendingDataArray.Num());
					for (const FPendingSpatialData& PendingData : PendingDataArray)
					{
						if (HasToBeUpdated(PendingData))
						{
							NewPendingData.Add(PendingData);
						}
					}
					PendingDataArray = MoveTemp(NewPendingData);
				}
			}
		}
	}

	void FPBDRigidsEvolutionBase::FlushInternalAccelerationQueue()
	{
		PruneInternalPendingData(*InternalAcceleration);
		// Could be linked by dependent task, but could start as soon a previous task is finished
		for (const TArray<FPendingSpatialData>& PendingDataArray : InternalAccelerationQueue.PendingDataArrays)
		{
			for (const FPendingSpatialData& PendingData : PendingDataArray)
			{
				ApplyParticlePendingData(PendingData, *InternalAcceleration, false, true);
			}
		}
		InternalAcceleration->SetSyncTimestamp(LatestExternalTimestampConsumed_Internal);
		InternalAccelerationQueue.Reset();
	}

	void FPBDRigidsEvolutionBase::FlushAsyncAccelerationQueue()
	{
		auto FlushAsyncAccelerationQueueLambda = [this]() 
		{
			for (const FPendingSpatialData& PendingData : AsyncAccelerationQueue.PendingData)
			{
				ApplyParticlePendingData(PendingData, *AsyncInternalAcceleration, true, false); //only the first queue needs to update the cached acceleration
				ApplyParticlePendingData(PendingData, *AsyncExternalAcceleration, false, false);
			}

			//NOTE: This assumes that we are never creating a PT particle that is replicated to GT
			//At the moment that is true, and it seems like we have enough mechanisms to avoid this direction
			//If we want to support that, the UniqueIndex must be kept around until GT goes away
			//This is hard to do, but would probably mean the ownership of the index is in the proxy
			for (FUniqueIdx UniqueIdx : UniqueIndicesPendingRelease)
			{
				ReleaseIdx(UniqueIdx);
			}
			UniqueIndicesPendingRelease.Reset();
			AsyncAccelerationQueue.Reset();
			AsyncInternalAcceleration->SetSyncTimestamp(LatestExternalTimestampConsumed_Internal);
			AsyncExternalAcceleration->SetSyncTimestamp(LatestExternalTimestampConsumed_Internal);
		};
		if (!Chaos::ShouldExecuteTasks(Particles.GetActiveDynamicMovingKinematicParticlesView().Num()))
		{
			FlushAsyncAccelerationQueueLambda();
		}
		else
		{
			TaskDispatcher.FlushAccelerationQueue(FlushAsyncAccelerationQueueLambda);
		}
	}

	//TODO: make static and _External suffix
	void FPBDRigidsEvolutionBase::FlushExternalAccelerationQueue(FAccelerationStructure& Acceleration, FPendingSpatialDataQueue& ExternalQueue)
	{
		//update structure with any pending operations. Note that we must keep those operations around in case next structure still hasn't consumed them (async mode)
		const int32 SyncTimestamp = Acceleration.GetSyncTimestamp();
		for (int32 Idx = ExternalQueue.PendingData.Num() - 1; Idx >=0; --Idx)
		{
			const FPendingSpatialData& SpatialData = ExternalQueue.PendingData[Idx];
			if(SpatialData.SyncTimestamp >= SyncTimestamp)
			{
				//operation still pending so update structure
				//note: do we care about roll over? if game ticks at 60fps we'd get 385+ days
				if(SpatialData.Operation == EPendingSpatialDataOperation::Delete)
				{
					const bool bExisted = Acceleration.RemoveElementFrom(SpatialData.AccelerationHandle,SpatialData.SpatialIdx);
					//ensure(bExisted);
				}
				else
				{
					FGeometryParticle* UpdateParticle = SpatialData.AccelerationHandle.GetExternalGeometryParticle_ExternalThread();
					FAABB3 WorldBounds;
					const bool bHasBounds = UpdateParticle->GetGeometry() && UpdateParticle->GetGeometry()->HasBoundingBox();
					if(bHasBounds)
					{
						TRigidTransform<FReal,3> WorldTM(UpdateParticle->X(),UpdateParticle->R());
						WorldBounds = UpdateParticle->GetGeometry()->BoundingBox().TransformedAABB(WorldTM);
					}
					const bool bExisted = Acceleration.UpdateElementIn(UpdateParticle,WorldBounds,bHasBounds,SpatialData.SpatialIdx);
					if (SpatialData.Operation == EPendingSpatialDataOperation::Add)
					{
						//ensure(bExisted == false); // TODO use this for ADD/UPDATE Audit
					}
					else
					{
						//ensure(bExisted == true); // TODO use this for ADD/UPDATE Audit						
					}
				}
			}
			else
			{
				//operation was already considered by sim, so remove it
				//going in reverse order so PendingData will stay valid
				ExternalQueue.Remove(SpatialData.UniqueIdx());
			}
		}
	}

	void FPBDRigidsEvolutionBase::WaitOnAccelerationStructure()
	{
		if (AccelerationStructureTaskComplete.GetReference())
		{
			FGraphEventArray ThingsToComplete;
			ThingsToComplete.Add(AccelerationStructureTaskComplete);
			QUICK_SCOPE_CYCLE_COUNTER(STAT_TPBDRigidsEvolutionBase_WaitAccelerationStructure);
			FTaskGraphInterface::Get().WaitUntilTasksComplete(ThingsToComplete);
		}
	}

	typename FPBDRigidsEvolutionBase::FAccelerationStructure* FPBDRigidsEvolutionBase::GetFreeSpatialAcceleration_Internal()
	{
		FAccelerationStructure* Structure;
		if(!ExternalStructuresPool.Dequeue(Structure))
		{
			AccelerationBackingBuffer.Add(SpatialCollectionFactory->CreateEmptyCollection());
			Structure = AccelerationBackingBuffer.Last().Get();
		}
		
		return Structure;
	}

	void FPBDRigidsEvolutionBase::FreeSpatialAcceleration_External(FAccelerationStructure* Structure)
	{
		//don't need to reset as rebuild task does that for us
		ExternalStructuresPool.Enqueue(Structure);
	}
	
	void FPBDRigidsEvolutionBase::ComputeIntermediateSpatialAcceleration(bool bBlock)
	{
		LLM_SCOPE(ELLMTag::ChaosAcceleration);
		SCOPE_CYCLE_COUNTER(STAT_ComputeIntermediateSpatialAcceleration);
		CHAOS_SCOPED_TIMER(ComputeIntermediateSpatialAcceleration);


		if (!AccelerationStructureTaskComplete)
		{
			//initial frame so make empty structures

			InternalAcceleration = GetFreeSpatialAcceleration_Internal();
			AsyncInternalAcceleration = GetFreeSpatialAcceleration_Internal();
			AsyncExternalAcceleration = GetFreeSpatialAcceleration_Internal();
			FlushInternalAccelerationQueue();
			
			TaskDispatcher.WaitAsyncQueueTask();

			// Give game thread an empty structure to pop
			ExternalStructuresQueue.Enqueue(AsyncExternalAcceleration);
			AsyncExternalAcceleration = GetFreeSpatialAcceleration_Internal();
		}

		if (bBlock)
		{
			WaitOnAccelerationStructure();
		}

		// Readability note:
		// AccelerationStructureTaskComplete It is a reference counted pointer to a TaskGraph event, that will be null if it is not set up
		bool AsyncComplete = !AccelerationStructureTaskComplete || AccelerationStructureTaskComplete->IsComplete();
		if (AsyncComplete && !IsResimming())
		{
			bool bNeedsReset = false;

			// only copy when the acceleration structures have completed time-slicing
			if (AccelerationStructureTaskComplete && AsyncInternalAcceleration->IsAllAsyncTasksComplete())
			{
				SCOPE_CYCLE_COUNTER(STAT_SwapAccelerationStructures);

				check(AsyncInternalAcceleration->IsAllAsyncTasksComplete());

				bNeedsReset = true;

				FlushAsyncAccelerationQueue();

				if (AccelerationStructureSplitStaticAndDynamic == 1)
				{
					FlushInternalAccelerationQueue();
				}
				else
				{
					// Other queues are no longer needed since we've flushed all operations and now have a pristine structure
					// Wait for the integration to finish before reseting the internal acceleration queue. 
					// Note: We could even not fill the Internal Acceleration Queue when only one acceleration structure 
					// is used (AccelerationStructureSplitStaticAndDynamic is disabled)
					TaskDispatcher.WaitIntegrationComplete();
					InternalAccelerationQueue.Reset();
				}
				TaskDispatcher.WaitAsyncQueueTask();

				// At this point the InternalAcceleration and the AsyncAcceleration structures will all be consistent with each other
				// (If FlushInternalAccelerationQueue was flushed above)
				// but the AsyncAcceleration now probably have fewer dirty elements (they were just rebuilt)

				//swap acceleration structure for new one
				std::swap(InternalAcceleration, AsyncInternalAcceleration);
				// The old InternalAcceleration is now in AsyncInternalAcceleration, and the constituent structures will be reused if no changes have been detected

				if (AccelerationStructureUseDynamicTree)
				{
					// Dynamic Acceleration structures were not built by the async task, so copy them where required
					CopyUnBuiltDynamicAccelerationStructures(SpatialAccelerationCache, InternalAcceleration, AsyncInternalAcceleration, AsyncExternalAcceleration);

					if(GAccelerationStructureCacheOverlappingLeaves)
					{
						// Caching of the overlapping leaves
						InternalAcceleration->CacheOverlappingLeaves();
					}
				}
				if (AccelerationStructureSplitStaticAndDynamic == 1)
				{
					// If the new InternalAcceleration structure have constituents with no changes, we can copy it to the  AsyncInternalAcceleration for reuse
					// This step can be left out to remove a copy here, at the cost of an unnecessary recalculation in the ChaosAccelerationStructureTask
					// For structures that don't change often both source and destination will be pristine and no copies will be made (most of the time)
					//CopyPristineAccelerationStructures(SpatialAccelerationCache, InternalAcceleration, AsyncInternalAcceleration, true);
				}

				//mark structure as ready for external thread
				ExternalStructuresQueue.Enqueue(AsyncExternalAcceleration);

				//get a new structure to work on while we wait for external thread to consume the one we just finished
				AsyncExternalAcceleration = GetFreeSpatialAcceleration_Internal();
			}
			else
			{
				FlushInternalAccelerationQueue();

				TaskDispatcher.WaitAsyncQueueTask();

				if (GAccelerationStructureCacheOverlappingLeaves  && AccelerationStructureUseDynamicTree)
				{
					// Caching of the overlapping leaves
					InternalAcceleration->CacheOverlappingLeaves();
				}
			}
			
			if (bCanStartAsyncTasks)
			{
				// we run the task for both starting a new accel structure as well as for the time-slicing
				bool ForceFullBuild = AsyncAccelerationQueue.Num() > AccelerationStructureTimeSlicingMaxQueueSizeBeforeForce;
				AccelerationStructureTaskComplete = TGraphTask<FChaosAccelerationStructureTask>::CreateTask().ConstructAndDispatchWhenReady(*SpatialCollectionFactory, SpatialAccelerationCache, AsyncInternalAcceleration, AsyncExternalAcceleration, ForceFullBuild, bIsSingleThreaded, bNeedsReset, &bAccelerationStructureTaskStarted, &bAccelerationStructureTaskSignalKill);
			}
		}
		else
		{
			FlushInternalAccelerationQueue();
			TaskDispatcher.WaitAsyncQueueTask();
		}
	}

	void FPBDRigidsEvolutionBase::CopyUnBuiltDynamicAccelerationStructures(const TMap<FSpatialAccelerationIdx, TUniquePtr<FSpatialAccelerationCache>>& SpatialAccelerationCache, FAccelerationStructure* InternalAcceleration, FAccelerationStructure* AsyncInternalAcceleration, FAccelerationStructure* AsyncExternalAcceleration)
	{
		for (const auto& Itr : SpatialAccelerationCache)
		{
			const FSpatialAccelerationIdx SpatialIdx = Itr.Key;
			auto* InternalAccelerationSubStructure = InternalAcceleration->GetSubstructure(SpatialIdx);
			if (!InternalAccelerationSubStructure)
			{
				ensure(false);
				continue;
			}

			if (!InternalAccelerationSubStructure->IsTreeDynamic())
			{
				continue;
			}

			auto* AsyncInternalAccelerationSubStructure = AsyncInternalAcceleration->GetSubstructure(SpatialIdx);
			if (!AsyncInternalAccelerationSubStructure)
			{
				ensure(false);
				continue;
			}

			check(AsyncInternalAccelerationSubStructure->IsTreeDynamic());
			// The good up to date data is in AsyncInternalAcceleration
			if (AsyncExternalAcceleration)
			{
				auto* ExternalAccelerationSubStructure = AsyncExternalAcceleration->GetSubstructure(SpatialIdx);
				if (!ExternalAccelerationSubStructure)
				{
					ensure(false);
					continue;
				}

				check(ExternalAccelerationSubStructure->IsTreeDynamic());
				ExternalAccelerationSubStructure->DeepAssign(*AsyncInternalAccelerationSubStructure);
			}
			
			InternalAcceleration->SwapSubstructure(*AsyncInternalAcceleration, SpatialIdx);
			//AsyncInternalAcceleration->GetSubstructure(SpatialIdx)->Reset(); // No need to reset
			
		}
	}

	// copy Pristine substructures from FromStructure to ToStructure, if the substructure is not already pristine
	// Assumption: Both structures represent the same state, one can just be dirtier than the other
	void FPBDRigidsEvolutionBase::CopyPristineAccelerationStructures(const TMap<FSpatialAccelerationIdx, TUniquePtr<FSpatialAccelerationCache>>& SpatialAccelerationCache, FAccelerationStructure* FromStructure, FAccelerationStructure* ToStructure, bool CheckPristine)
	{
		
		for (const auto& Itr : SpatialAccelerationCache)
		{
			const FSpatialAccelerationIdx SpatialIdx = Itr.Key;
			auto* FromSubStructure = FromStructure->GetSubstructure(SpatialIdx);
			auto* ToSubStructure = ToStructure->GetSubstructure(SpatialIdx);

			if (!FromSubStructure || !ToSubStructure)
			{
				ensure(false);
				continue;
			}

			if (!CheckPristine || (ToSubStructure->ShouldRebuild() && !FromSubStructure->ShouldRebuild()))
			{
				ToSubStructure->DeepAssign(*FromSubStructure);
			}
		}
		// This is a pretty shallow copy: It won't copy anything that we copied above
		ToStructure->DeepAssign(*FromStructure);
	}

	void FPBDRigidsEvolutionBase::UpdateExternalAccelerationStructure_External(
		ISpatialAccelerationCollection<FAccelerationStructureHandle, FReal, 3>*& StructToUpdate, FPendingSpatialDataQueue& PendingExternal)
	{
		DECLARE_SCOPE_CYCLE_COUNTER(TEXT("CreateExternalAccelerationStructure"), STAT_CreateExternalAccelerationStructure, STATGROUP_Physics);
		LLM_SCOPE(ELLMTag::ChaosAcceleration);

		FAccelerationStructure* NewStruct = nullptr;
		while(ExternalStructuresQueue.Dequeue(NewStruct))	//get latest structure
		{
			//free old struct
			if(StructToUpdate)
			{
				FreeSpatialAcceleration_External(StructToUpdate);
			}

			StructToUpdate = NewStruct;
		}

		if (ensure(StructToUpdate) && NewStruct != nullptr)
		{
			FlushExternalAccelerationQueue(*StructToUpdate, PendingExternal);
		}
	}

	void FPBDRigidsEvolutionBase::FlushSpatialAcceleration()
	{
		//force build acceleration structure with latest data
		ComputeIntermediateSpatialAcceleration(true);
		ComputeIntermediateSpatialAcceleration(true);	//having to do it multiple times because of the various caching involved over multiple frames.
		ComputeIntermediateSpatialAcceleration(true);
	}

	void FPBDRigidsEvolutionBase::RebuildSpatialAccelerationForPerfTest()
	{
		WaitOnAccelerationStructure();

		ParticleToCacheInnerIdx.Reset();
		AsyncAccelerationQueue.Reset();
		InternalAccelerationQueue.Reset();

		AccelerationStructureTaskComplete = nullptr;
		const auto& NonDisabled = Particles.GetNonDisabledView();
		for (auto& Particle : NonDisabled)
		{
			DirtyParticle(Particle);
		}

		FlushSpatialAcceleration();
	}

	void FPBDRigidsEvolutionBase::ReleaseIdx(FUniqueIdx Idx)
	{
		PendingReleaseIndices.Add(Idx);
	}

	void FPBDRigidsEvolutionBase::ReleasePendingIndices()
	{
		for (FUniqueIdx Idx : PendingReleaseIndices)
		{
			Particles.GetUniqueIndices().ReleaseIdx(Idx);
		}
		PendingReleaseIndices.Reset();
	}


	void FPBDRigidsEvolutionBase::Serialize(FChaosArchive& Ar)
	{
		ensure(false);	//disabled transient data serialization. Need to rethink
		int32 DefaultBroadphaseType = BroadPhaseConfig.BroadphaseType;

		Ar.UsingCustomVersion(FExternalPhysicsCustomObjectVersion::GUID);
		if (Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) >= FExternalPhysicsCustomObjectVersion::SerializeBroadphaseType)
		{
			Ar << BroadPhaseConfig.BroadphaseType;
		}
		else
		{
			//older archives just assume type 3
			BroadPhaseConfig.BroadphaseType = 3;
		}

		Particles.Serialize(Ar);

		if (Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) >= FExternalPhysicsCustomObjectVersion::SerializeEvolutionBV)
		{
			if (Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) >= FExternalPhysicsCustomObjectVersion::FlushEvolutionInternalAccelerationQueue)
			{
				FlushInternalAccelerationQueue();
			}

			if (Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) < FExternalPhysicsCustomObjectVersion::SerializeMultiStructures)
			{
				//old path assumes single sub-structure
				TUniquePtr<ISpatialAcceleration<FAccelerationStructureHandle, FReal, 3>> SubStructure;
				if (!Ar.IsLoading())
				{
					SubStructure = InternalAcceleration->RemoveSubstructure(FSpatialAccelerationIdx{ 0,0 });
					Ar << SubStructure;
					InternalAcceleration->AddSubstructure(MoveTemp(SubStructure), 0, 0);
				}
				else
				{
					Ar << SubStructure;
					AccelerationBackingBuffer.Add(CreateNewSpatialStructureFromSubStructure(MoveTemp(SubStructure)));
					InternalAcceleration = AccelerationBackingBuffer.Last().Get();
				}
			}
			else
			{
				TUniquePtr<FAccelerationStructure> InternalUnique;
				SpatialCollectionFactory->Serialize(InternalUnique, Ar);
				InternalAcceleration = InternalUnique.Get();
				AccelerationBackingBuffer.Add(MoveTemp(InternalUnique));
			}

			/*if (Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) < FExternalPhysicsCustomObjectVersion::FlushEvolutionInternalAccelerationQueue)
			{
				SerializePendingMap(Ar, InternalAccelerationQueue);
				SerializePendingMap(Ar, AsyncAccelerationQueue);
				SerializePendingMap(Ar, ExternalAccelerationQueue);
			}*/
		}
		else if (Ar.IsLoading())
		{
			AccelerationStructureTaskComplete = nullptr;
			for (auto& Particle : Particles.GetNonDisabledView())
			{
				Particle.SetSpatialIdx(FSpatialAccelerationIdx{ 0,0 });
				DirtyParticle(Particle);
			}

			FlushSpatialAcceleration();
		}

		BroadPhaseConfig.BroadphaseType = DefaultBroadphaseType;
	}

	void FPBDRigidsEvolutionBase::SetParticlesObjectState(TArrayView<FPBDRigidParticleHandle*> RigidParticles, 
		const EObjectStateType ObjectState, const bool bUpdateViews)
	{
		if (RigidParticles.IsEmpty())
		{
			return;
		}
		const bool bIsDynamic = (ObjectState == EObjectStateType::Dynamic) || (ObjectState == EObjectStateType::Sleeping);
		const bool bIsSleeping = (ObjectState == EObjectStateType::Sleeping);
		
		// Sorting algorithm are modifying the array view in place and is O(N-ParticleOffset), N being the view size
		int32 ParticleOffset = 0;
		
		// The particles in [0-ParticleOffset) will be the valid ones the [ParticleOffset,NumParticles) will be the others
		Private::ReorderParticleView(RigidParticles, [](FPBDRigidParticleHandle* RigidParticle) -> bool 
			{ return (RigidParticle != nullptr);},ParticleOffset);
		
		const int32 NumValids = ParticleOffset;
		if (NumValids > 0)
		{
			TArrayView<FPBDRigidParticleHandle*> ValidParticles(&RigidParticles[0], NumValids);
			ParticleOffset = 0;

			// The particles in [0-ParticleOffset) will be the clustered one whereas the [ParticleOffset,NumParticles) will be the others
			Private::ReorderParticleView(ValidParticles, [](FPBDRigidParticleHandle* RigidParticle) -> bool
				{ return (RigidParticle->CastToClustered() != nullptr);}, ParticleOffset);
		
			// Move particles to clustered SOA
			const int32 NumClustered = ParticleOffset;
			if(NumClustered > 0)
			{
				TArrayView<FPBDRigidParticleHandle*> ClusteredParticles(&ValidParticles[0], NumClustered);
				Particles.SetClusteredParticlesSOA(ClusteredParticles, ObjectState, false);
			}
		
			// Move particles to dynamics SOA
			const int32 NumDynamics = ValidParticles.Num()-NumClustered;
			if (NumDynamics > 0)
			{
				TArrayView<FPBDRigidParticleHandle*> DynamicParticles(&ValidParticles[NumClustered], NumDynamics);
				Particles.SetDynamicParticlesSOA(DynamicParticles, ObjectState, false);
			}
		}
		
		ParticleOffset = 0;
			
		// The particles in [0-ParticleOffset) will be the enabled ones switching to the new state
		Private::ReorderParticleView(RigidParticles, [&ObjectState](FPBDRigidParticleHandle* RigidParticle) -> bool 
			{ return RigidParticle && (RigidParticle->ObjectState() != ObjectState && !RigidParticle->Disabled());}, ParticleOffset);	
		
		const int32 NumTransients = ParticleOffset;
		if (NumTransients > 0)
		{
			// Mark transient particles dirty
			TArrayView<FPBDRigidParticleHandle*> TransientParticles(&RigidParticles[0], NumTransients);
			if (ObjectState != EObjectStateType::Dynamic)
			{
				Particles.MarkTransientDirtyParticles(TransientParticles, false);
			}
			
			for (FPBDRigidParticleHandle* Particle : TransientParticles)
			{
				if (Particle)
				{
					// Reset the sleep counter when the state changes
					// NOTE: this sleep counter is only used for particles
					// that are not in the graph (have no constraints on them)
					if (bIsDynamic)
					{
						Particle->SetSleepCounter(0);
					}

					// If all particles in an island are explicitly put to sleep, the island and all its
					// constraints need to sleep at the start of the next tick. This is because we don't
					// detect collisions on sleeping particles, and we destroy collisions that are 
					// awake and were not updated this tick.
					if (bIsSleeping)
					{
						IslandManager.SleepParticle(Particle);
					}

					// If we are not kinematic, the MovingKinematic flag should be cleared
					// If we are kinematic and are (or become) moving, the flag will be updated in ApplyKinematicTargets
					// Either way, we can reset the flag here
					Particle->ClearIsMovingKinematic();
				}
			}
		}
		
		if(bUpdateViews)
		{ 
			Particles.UpdateDirtyViews();
		}
		
		// Modify particle object state
		int32 ParticleIndex = 0;
		for (FPBDRigidParticleHandle* Particle : RigidParticles)
		{
			if (Particle)
			{
				const bool bWasDynamic = (Particle->ObjectState() == EObjectStateType::Dynamic) || (Particle->ObjectState() == EObjectStateType::Sleeping);
			
				// If we are now dynamic and enabled, we need to be in the graph if not already there. We may not be
				// because Kinematic particles are only in the graph if referenced by a constraint.
				if (bIsDynamic && !bWasDynamic && !Particle->Disabled())
				{
					IslandManager.AddParticle(Particle);
				}

				// If we switched to/from dynamic the inertia conditioning may need to be refreshed
				if (Particle->InertiaConditioningEnabled() && (bWasDynamic != bIsDynamic))
				{
					Particle->SetInertiaConditioningDirty();
				}
				
				// Change the low level object state 
				Particle->SetObjectStateLowLevel(ObjectState);
			}
		}
	}
	
	void FPBDRigidsEvolutionBase::SetParticleObjectState(FPBDRigidParticleHandle* Particle, EObjectStateType ObjectState)
	{
		TArrayView<FPBDRigidParticleHandle*> RigidParticles(&Particle, 1);
		SetParticlesObjectState(RigidParticles, ObjectState, true);
	}

	void FPBDRigidsEvolutionBase::WakeParticle(FPBDRigidParticleHandle* Particle)
	{
		if (Particle->IsSleeping())
		{
			// Set to dynamic - this will also wake the particle's island
			SetParticleObjectState(Particle, EObjectStateType::Dynamic);
		}

		// Explicitly waking a particle should reset any sleep state, even if we are already awake.
		// E.g., this will reset the sleep counter to prevent the particle from immediately sleeping again.
		// NOTE: This also allows us to "wake" a kinematic which will wake all islands that the kinematic is in.
		IslandManager.WakeParticleIslands(Particle);

		IslandManager.FlagParticleUpdated(Particle);
	}

	void FPBDRigidsEvolutionBase::SetParticleSleepType(FPBDRigidParticleHandle* Particle, ESleepType InSleepType)
	{
		//ESleepType InitialSleepType = Particle->SleepType();
		const EObjectStateType InitialState = Particle->ObjectState();
		Particle->SetSleepType(InSleepType);
		const EObjectStateType ObjectState = Particle->ObjectState();

		if (FPBDRigidClusteredParticleHandle* ClusteredParticle = Particle->CastToClustered())
		{
			Particles.SetClusteredParticleSOA(ClusteredParticle);
		}
		else
		{
			Particles.SetDynamicParticleSOA(Particle);
		}

		if (InitialState != ObjectState)
		{
			if(ObjectState != EObjectStateType::Dynamic)
			{
				// even though we went to sleep, we should still report info back to GT
				Particles.MarkTransientDirtyParticle(Particle);
			}
		}

		// Refresh cached sleep thresholds on the island node so that the island manager respects the new sleep type
		IslandManager.UpdateParticleMaterial(Particle);
	}

	void FPBDRigidsEvolutionBase::DisableParticles(const TSet<FGeometryParticleHandle*>& InParticles)
	{
		for (FGeometryParticleHandle* Particle : InParticles)
		{
			Particles.DisableParticle(Particle);
			RemoveParticleFromAccelerationStructure(*Particle);
			DisableConstraints(Particle);
			IslandManager.RemoveParticle(Particle);
		}
	}

	void FPBDRigidsEvolutionBase::DisableParticleWithRemovalEvent(FGeometryParticleHandle* Particle)
	{
		if (Particle == nullptr)
		{
			return;
		}
		
		// Record removal for event generation
		FRemovalData& Removal = MAllRemovals.AddDefaulted_GetRef();
		Removal.Proxy = Particle->PhysicsProxy();
		Removal.Location = Particle->GetX();
		
		if (Chaos::FPBDRigidParticleHandle* RigidParticle = Particle->CastToRigidParticle())
		{
			Removal.Mass = RigidParticle->M();
		}
		else
		{
			Removal.Mass = 0.0;
		}
		
		if (Particle->GetGeometry() && Particle->GetGeometry()->HasBoundingBox())
		{
			Removal.BoundingBox = Particle->GetGeometry()->BoundingBox();
		}
		
		DisableParticle(Particle);
	}

	const FChaosPhysicsMaterial* FPBDRigidsEvolutionBase::GetFirstPhysicsMaterial(const FGeometryParticleHandle* Particle) const
	{
		return Private::GetFirstPhysicsMaterial(Particle, &PhysicsMaterials, &PerParticlePhysicsMaterials, &SolverPhysicsMaterials);
	}

#if CHAOS_EVOLUTION_COLLISION_TESTMODE
	void FPBDRigidsEvolutionBase::TestModeStep()
	{
		if (CVars::Chaos_Solver_TestMode_Step > 0)
		{
			TestModeData.Reset();
			--CVars::Chaos_Solver_TestMode_Step;
		}
	}

	void FPBDRigidsEvolutionBase::TestModeParticleDisabled(FGeometryParticleHandle* Particle)
	{
		if (FPBDRigidParticleHandle* Rigid = Particle->CastToRigidParticle())
		{
			TestModeData.Remove(Rigid);
		}
	}

	void FPBDRigidsEvolutionBase::TestModeSaveParticles()
	{
		if (!CVars::bChaos_Solver_TestMode_Enabled)
		{
			return;
		}

		for (auto& Rigid : Particles.GetNonDisabledDynamicView())
		{
			TestModeSaveParticle(Rigid.Handle());
		}
	}

	void FPBDRigidsEvolutionBase::TestModeSaveParticle(FGeometryParticleHandle* Particle)
	{
		if (!CVars::bChaos_Solver_TestMode_Enabled)
		{
			return;
		}

		if (FPBDRigidParticleHandle* Rigid = Particle->CastToRigidParticle())
		{
			FTestModeParticleData* Data = TestModeData.Find(Rigid);
			if (Data == nullptr)
			{
				TestModeData.Add(Rigid);
				TestModeUpdateSavedParticle(Particle);
			}
		}
	}

	void FPBDRigidsEvolutionBase::TestModeUpdateSavedParticle(FGeometryParticleHandle* Particle)
	{
		if (!CVars::bChaos_Solver_TestMode_Enabled)
		{
			return;
		}

		if (FPBDRigidParticleHandle* Rigid = Particle->CastToRigidParticle())
		{
			FTestModeParticleData* Data = TestModeData.Find(Rigid);
			if (Data != nullptr)
			{
				Data->X = Rigid->GetX();
				Data->P = Rigid->GetP();
				Data->R = Rigid->GetR();
				Data->Q = Rigid->GetQ();
				Data->V = Rigid->GetV();
				Data->W = Rigid->GetW();
			}
		}
	}

	void FPBDRigidsEvolutionBase::TestModeRestoreParticles()
	{
		if (!CVars::bChaos_Solver_TestMode_Enabled)
		{
			return;
		}

		for (auto& Rigid : Particles.GetNonDisabledDynamicView())
		{
			// If this is the first time we have seen this particle, save its data, otherwise restore it from the cache
			FTestModeParticleData* Data = TestModeData.Find(Rigid.Handle());
			if (Data != nullptr)
			{
				TestModeRestoreParticle(Rigid.Handle());
			}
		}
	}

	void FPBDRigidsEvolutionBase::TestModeRestoreParticle(FGeometryParticleHandle* Particle)
	{
		if (!CVars::bChaos_Solver_TestMode_Enabled)
		{
			return;
		}

		if (FPBDRigidParticleHandle* Rigid = Particle->CastToRigidParticle())
		{
			FTestModeParticleData* Data = TestModeData.Find(Rigid);
			if (Data != nullptr)
			{
				Rigid->SetX(Data->X);
				Rigid->SetP(Data->P);
				Rigid->SetR(Data->R);
				Rigid->SetQ(Data->Q);
				Rigid->SetV(Data->V);
				Rigid->SetW(Data->W);
			}
		}
	}
#endif

}
