// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/PBDRigidsEvolutionGBF.h"
#include "Chaos/Collision/SimSweep.h"
#include "Chaos/Defines.h"
#include "Chaos/Evolution/SolverBodyContainer.h"
#include "Chaos/Framework/HashMappedArray.h"
#include "Chaos/Framework/Parallel.h"
#include "Chaos/ImplicitObjectTransformed.h"
#include "Chaos/ImplicitObjectUnion.h"
#include "Chaos/MassConditioning.h"
#include "Chaos/MidPhaseModification.h"
#include "Chaos/PBDCollisionConstraints.h"
#include "Chaos/PerParticleEtherDrag.h"
#include "Chaos/PerParticleEulerStepVelocity.h"
#include "Chaos/PerParticleGravity.h"
#include "Chaos/PerParticleInitForce.h"
#include "Chaos/PerParticlePBDEulerStep.h"
#include "Chaos/PerParticlePBDGroundConstraint.h"
#include "Chaos/PerParticlePBDUpdateFromDeltaPosition.h"
#include "ChaosStats.h"
#include "Chaos/EvolutionResimCache.h"
#include "Chaos/ResimConsoleVariables.h"

#include "ProfilingDebugging/ScopedTimers.h"
#include "Chaos/DebugDrawQueue.h"
#include "Misc/ScopeLock.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"

#include "ChaosVisualDebugger/ChaosVisualDebuggerTrace.h"

#include "ChaosDebugDraw/ChaosDDContext.h"
#include "ChaosDebugDraw/ChaosDDScene.h"
#include "ChaosDebugDraw/ChaosDDTimeline.h"

namespace Chaos
{
#if !UE_BUILD_SHIPPING
	CHAOS_API bool bPendingHierarchyDump = false; 
#else
	const bool bPendingHierarchyDump = false;
#endif

	namespace CVars
	{
		// Debug/testing MACD
		extern bool bChaosUseMACD;
		extern bool bChaosForceMACD;

		FRealSingle HackMaxAngularVelocity = 1000.f;
		FAutoConsoleVariableRef CVarHackMaxAngularVelocity(TEXT("p.HackMaxAngularVelocity"), HackMaxAngularVelocity, TEXT("Max cap on angular velocity: rad/s. This is only a temp solution and should not be relied on as a feature. -1.f to disable"));

		FRealSingle HackMaxVelocity = -1.f;
		FAutoConsoleVariableRef CVarHackMaxVelocity(TEXT("p.HackMaxVelocity2"), HackMaxVelocity, TEXT("Max cap on velocity: cm/s. This is only a temp solution and should not be relied on as a feature. -1.f to disable"));


		int DisableThreshold = 5;
		FAutoConsoleVariableRef CVarDisableThreshold(TEXT("p.DisableThreshold2"), DisableThreshold, TEXT("Disable threshold frames to transition to sleeping"));

		int CollisionDisableCulledContacts = 0;
		FAutoConsoleVariableRef CVarDisableCulledContacts(TEXT("p.CollisionDisableCulledContacts"), CollisionDisableCulledContacts, TEXT("Allow the PBDRigidsEvolutionGBF collision constraints to throw out contacts mid solve if they are culled."));

		FRealSingle SmoothedPositionLerpRate = 0.3f;
		FAutoConsoleVariableRef CVarSmoothedPositionLerpRate(TEXT("p.Chaos.SmoothedPositionLerpRate"), SmoothedPositionLerpRate, TEXT("The interpolation rate for the smoothed position calculation. Used for sleeping."));

		int DisableParticleUpdateVelocityParallelFor = 0;
		FAutoConsoleVariableRef CVarDisableParticleUpdateVelocityParallelFor(TEXT("p.DisableParticleUpdateVelocityParallelFor"), DisableParticleUpdateVelocityParallelFor, TEXT("Disable Particle Update Velocity ParallelFor and run the update on a single thread"));

		// NOTE: If we have mrope than 1 CCD iteration (ChaosCollisionCCDConstraintMaxProcessCount), the tight bounding box will cause us to miss secondary CCD collisions if the first one(s) result in a change in direction
		bool bChaosCollisionCCDUseTightBoundingBox = true;
		FAutoConsoleVariableRef  CVarChaosCollisionCCDUseTightBoundingBox(TEXT("p.Chaos.Collision.CCD.UseTightBoundingBox"), bChaosCollisionCCDUseTightBoundingBox , TEXT(""));

		// Collision Solver Type (see ECollisionSolverType)
		int32 ChaosSolverCollisionSolverType = -1;
		FAutoConsoleVariableRef CVarChaosSolverCollisionSolverType(TEXT("p.Chaos.Solver.Collision.SolverType"), ChaosSolverCollisionSolverType, TEXT("-1: Use default (Gauss Seidel); 0: Gauss Seidel; 1: Gauss Seidel SOA 2: Partial Jacobi"));

		int32 ChaosSolverCollisionPriority = 0;
		FAutoConsoleVariableRef CVarChaosSolverCollisionPriority(TEXT("p.Chaos.Solver.Collision.Priority"), ChaosSolverCollisionPriority, TEXT("Set constraint priority. Larger values are evaluated later [def:0]"));

		int32 ChaosSolverJointPriority = 0;
		FAutoConsoleVariableRef CVarChaosSolverJointPriority(TEXT("p.Chaos.Solver.Joint.Priority"), ChaosSolverJointPriority, TEXT("Set constraint priority. Larger values are evaluated later [def:0]"));

		int32 ChaosSolverSuspensionPriority = 0;
		FAutoConsoleVariableRef CVarChaosSolverSuspensionPriority(TEXT("p.Chaos.Solver.Suspension.Priority"), ChaosSolverSuspensionPriority, TEXT("Set constraint priority. Larger values are evaluated later [def:0]"));

		int32 ChaosSolverCharacterGroundConstraintPriority = 0;
		FAutoConsoleVariableRef CVarChaosSolverChaosCharacterGroundConstraintPriority(TEXT("p.Chaos.Solver.CharacterGroundConstraint.Priority"), ChaosSolverCharacterGroundConstraintPriority, TEXT("Set constraint priority. Larger values are evaluated later [def:0]"));

		bool DoTransferJointConstraintCollisions = true;
		FAutoConsoleVariableRef CVarDoTransferJointConstraintCollisions(TEXT("p.Chaos.Solver.Joint.TransferCollisions"), DoTransferJointConstraintCollisions, TEXT("Allows joints to apply collisions to the parent from the child when the Joints TransferCollisionScale is not 0 [def:true]"));

		int32 TransferCollisionsLimit = INT_MAX;
		FAutoConsoleVariableRef CVarTransferCollisionsMultiply(TEXT("p.Chaos.Solver.Joint.TransferCollisionsLimit"), TransferCollisionsLimit, TEXT("Maximum number of constraints that are allowed to transfer to the parent. Lowering this will improve performance but reduce accuracy. [def:INT_MAX]"));

		FRealSingle TransferCollisionsKinematicScale = 1.0f;
		FAutoConsoleVariableRef CVarTransferCollisionsKinematicScale(TEXT("p.Chaos.Solver.Joint.TransferCollisionsKinematicScale"), TransferCollisionsKinematicScale, TEXT("Scale to apply to collision transfers between kinematic bodies [def:1.0]"));

		FRealSingle TransferCollisionsStiffnessClamp = 1.0f;
		FAutoConsoleVariableRef CVarTransferCollisionsStiffnessClamp(TEXT("p.Chaos.Solver.Joint.TransferCollisionsStiffnessClamp"), TransferCollisionsStiffnessClamp, TEXT("Clamp of maximum value of the stiffness clamp[def:1.0]"));

		bool TransferCollisionsDebugTestAgainstMaxClamp = false;
		FAutoConsoleVariableRef CVarTransferCollisionsDebugTestAgainstMaxClamp(TEXT("p.Chaos.Solver.Joint.TransferCollisionsDebugTestAgainstMaxClamp"), TransferCollisionsDebugTestAgainstMaxClamp, TEXT("Force all joint collision constraint settings to max clamp value to validate stability [def:false]"));

		bool DoFinalProbeNarrowPhase = true;
		FAutoConsoleVariableRef CVarDoFinalProbeNarrowPhase(TEXT("p.Chaos.Solver.DoFinalProbeNarrowPhase"), DoFinalProbeNarrowPhase, TEXT(""));

		// Gets incremented when one of the Inertia Conditioning CVars changes so we can refresh the system.
		// Uses a change counter rather than a boolean to handle multiple Evolutions (e.g., Client+Server PIE)
		int32 ChaosSolverInertiaConditioningEpoch = 0;
		FConsoleVariableDelegate OnInertiaConditioningCVarChangedDelegate = FConsoleVariableDelegate::CreateLambda([](IConsoleVariable*) { ++ChaosSolverInertiaConditioningEpoch; });

		// Enable inertia conditioning for joint and collisions stabilization for small and thin objects
		bool bChaosSolverInertiaConditioningEnabled = true;
		FAutoConsoleVariableRef  CVarChaosSolverInertiaConditioningEnabled(
			TEXT("p.Chaos.Solver.InertiaConditioning.Enabled"), 
			bChaosSolverInertiaConditioningEnabled, 
			TEXT("Enable/Disable constraint stabilization through inertia conditioning"), 
			OnInertiaConditioningCVarChangedDelegate);

		// The largest joint error we expect to resolve in a moderately stable way
		FRealSingle ChaosSolverInertiaConditioningDistance = 20;
		FAutoConsoleVariableRef  CVarChaosSolverInertiaConditioningDistance(
			TEXT("p.Chaos.Solver.InertiaConditioning.Distance"), 
			ChaosSolverInertiaConditioningDistance, 
			TEXT("An input to inertia conditioning system. The joint distance error which needs to be stable (generate a low rotation)."), 
			OnInertiaConditioningCVarChangedDelegate);

		// The ratio of joint error correction that comes from particle rotation versus translation.
		// NOTE: A value of 2.5 means that spheres (and cubes) are not effected by inertia conditioning (unless they have constraints outside their bounds).
		FRealSingle ChaosSolverInertiaConditioningRotationRatio = 2.5f;
		FAutoConsoleVariableRef  CVarChaosSolverInertiaConditioningRotationRatio(
			TEXT("p.Chaos.Solver.InertiaConditioning.RotationRatio"), 
			ChaosSolverInertiaConditioningRotationRatio, 
			TEXT("An input to inertia conditioning system. The maximum ratio of joint correction from rotation versus translation"), 
			OnInertiaConditioningCVarChangedDelegate);

		// If > 1, limits the inverse inertia components to be mo more than this multiple of the smallest component. Makes the objects more round. Set to 0 to disable.
		FRealSingle ChaosSolverMaxInvInertiaComponentRatio = 0;
		FAutoConsoleVariableRef  CVarChaosSolverInertiaConditioningMaxInvInertiaComponentRatio(
			TEXT("p.Chaos.Solver.InertiaConditioning.MaxInvInertiaComponentRatio"),
			ChaosSolverMaxInvInertiaComponentRatio,
			TEXT("An input to inertia conditioning system. The largest inertia component must be at least least multiple of the smallest component"), 
			OnInertiaConditioningCVarChangedDelegate);

		// NOTE: ChaosInertiaConditioningInvInertiaTolerance of 1e-4 effectively disables inertia conditioning for medium and larger object.
		// If you want inertia conditioning on large objects, make ChaosInertiaConditioningInvInertiaTolerance very small (say, 1e-30).
		FInertiaConditioningTolerances ChaosInertiaConditioningTolerances = FInertiaConditioningTolerances();
		FAutoConsoleVariableRef  CVarChaosInertiaConditioningInvMassTolerance(
			TEXT("p.Chaos.InertiaConditioning.InvMassTolerance"),
			ChaosInertiaConditioningTolerances.InvMassTolerance,
			TEXT("Inertia conditioning is disabled for objects with inverse mass less than this."),
			OnInertiaConditioningCVarChangedDelegate);
		FAutoConsoleVariableRef  CVarChaosInertiaConditioningInvInertiaTolerance(
			TEXT("p.Chaos.InertiaConditioning.InvInertiaTolerance"),
			ChaosInertiaConditioningTolerances.InvInertiaTolerance,
			TEXT("Inertia conditioning is disabled for objects with inverse inertia less than this. The default value disables Inertia Conditioing for mediums to large objects."),
			OnInertiaConditioningCVarChangedDelegate);
		FAutoConsoleVariableRef  CVarChaosInertiaConditioningExtentTolerance(
			TEXT("p.Chaos.InertiaConditioning.ExtentTolerance"),
			ChaosInertiaConditioningTolerances.ExtentTolerance,
			TEXT("Inertia conditioning is disabled for objects with dimensions smaller than this."),
			OnInertiaConditioningCVarChangedDelegate);

		// Collision modifiers are run after the CCD rewind and manifold regeneration because the CCD "contact" data is only used to rewind the particle 
		// and we regenerate the manifold at that new position. Running the modifier callback before CCD rewind would mean that the user does not have 
		// data related to the actual contacts that will be resolved, and the positions and normals may be misleading. 
		// This cvar allows use to run the modifiers before CCD rewind which is how it used to be, just in case.
		// The new way to disable CCD contacts (pre-rewind) is to do it in a MidPhase- or CCD-Modifier callbacks. See FCCDModifier.
		bool bChaosCollisionModiferBeforeCCD = false;
		FAutoConsoleVariableRef  CVarChaosCollisionModiferBeforeCCD(TEXT("p.Chaos.Solver.CollisionModifiersBeforeCCD"), bChaosCollisionModiferBeforeCCD, TEXT("True: run the collision modifiers before CCD rewind is applied; False(default): run modifiers after CCD rewind. See comments in code."));

		// Whether the constraint graph retains state between ticks. It will be very expensive with this disabled...
		bool bChaosSolverPersistentGraph = true;
		FAutoConsoleVariableRef CVarChaosSolverPersistentGraph(TEXT("p.Chaos.Solver.PersistentGraph"), bChaosSolverPersistentGraph, TEXT(""));

		// Determines what happens when two one-way particles collide
		// See EOneWayInteractionPairCollisionMode
		int32 ChaosOneWayInteractionPairCollisionMode = (int32)EOneWayInteractionPairCollisionMode::SphereCollision;
		FAutoConsoleVariableRef CVarChaosIgnoreOneWayPairCollisions(TEXT("p.Chaos.Solver.OneWayPairCollisionMode"), ChaosOneWayInteractionPairCollisionMode, TEXT("How to treat collisions between two one-way interaction particles. See EOneWayInteractionPairCollisionMode (0: Ignore collisions; 1: Collide as normal; 2: Collide as spheres)"));

		bool bChaosRigids_UseSimdForJointsSolver = true;
		FAutoConsoleVariableRef  CVarChaosImmPhysUseSimdForLinearSolver(TEXT("p.Chaos.Solver.Joint.UseSimd"), bChaosRigids_UseSimdForJointsSolver, TEXT("Enable/Disable SIMD on the linear joint solver"));

		DECLARE_CYCLE_STAT(TEXT("FPBDRigidsEvolutionGBF::AdvanceOneTimeStep"), STAT_Evolution_AdvanceOneTimeStep, STATGROUP_Chaos);
		DECLARE_CYCLE_STAT(TEXT("FPBDRigidsEvolutionGBF::UnclusterUnions"), STAT_Evolution_UnclusterUnions, STATGROUP_Chaos);
		DECLARE_CYCLE_STAT(TEXT("FPBDRigidsEvolutionGBF::Integrate"), STAT_Evolution_Integrate, STATGROUP_Chaos);
		DECLARE_CYCLE_STAT(TEXT("FPBDRigidsEvolutionGBF::KinematicTargets"), STAT_Evolution_KinematicTargets, STATGROUP_Chaos);
		DECLARE_CYCLE_STAT(TEXT("FPBDRigidsEvolutionGBF::PreIntegrateCallback"), STAT_Evolution_PreIntegrateCallback, STATGROUP_Chaos);
		DECLARE_CYCLE_STAT(TEXT("FPBDRigidsEvolutionGBF::PostIntegrateCallback"), STAT_Evolution_PostIntegrateCallback, STATGROUP_Chaos);
		DECLARE_CYCLE_STAT(TEXT("FPBDRigidsEvolutionGBF::PreSolveCallback"), STAT_Evolution_PreSolveCallback, STATGROUP_Chaos);
		DECLARE_CYCLE_STAT(TEXT("FPBDRigidsEvolutionGBF::PostSolveCallback"), STAT_Evolution_PostSolveCallback, STATGROUP_Chaos);
		DECLARE_CYCLE_STAT(TEXT("FPBDRigidsEvolutionGBF::CCDModifierCallback"), STAT_Evolution_CCDModifierCallback, STATGROUP_Chaos);
		DECLARE_CYCLE_STAT(TEXT("FPBDRigidsEvolutionGBF::CollisionModifierCallback"), STAT_Evolution_CollisionModifierCallback, STATGROUP_Chaos);
		DECLARE_CYCLE_STAT(TEXT("FPBDRigidsEvolutionGBF::MidPhaseModifierCallback"), STAT_Evolution_MidPhaseModifierCallback, STATGROUP_Chaos);
		DECLARE_CYCLE_STAT(TEXT("FPBDRigidsEvolutionGBF::CCD"), STAT_Evolution_CCD, STATGROUP_Chaos);
		DECLARE_CYCLE_STAT(TEXT("FPBDRigidsEvolutionGBF::CCDCorrection"), STAT_Evolution_CCDCorrection, STATGROUP_Chaos);
		DECLARE_CYCLE_STAT(TEXT("FPBDRigidsEvolutionGBF::GraphColor"), STAT_Evolution_GraphColor, STATGROUP_Chaos);
		DECLARE_CYCLE_STAT(TEXT("FPBDRigidsEvolutionGBF::BuildGroups"), STAT_Evolution_BuildGroups, STATGROUP_Chaos);
		DECLARE_CYCLE_STAT(TEXT("FPBDRigidsEvolutionGBF::DetectCollisions"), STAT_Evolution_DetectCollisions, STATGROUP_Chaos);
		DECLARE_CYCLE_STAT(TEXT("FPBDRigidsEvolutionGBF::TransferJointCollisions"), STAT_Evolution_TransferJointCollisions, STATGROUP_Chaos);
		DECLARE_CYCLE_STAT(TEXT("FPBDRigidsEvolutionGBF::PostDetectCollisionsCallback"), STAT_Evolution_PostDetectCollisionsCallback, STATGROUP_Chaos);
		DECLARE_CYCLE_STAT(TEXT("FPBDRigidsEvolutionGBF::UpdateConstraintPositionBasedState"), STAT_Evolution_UpdateConstraintPositionBasedState, STATGROUP_Chaos);
		DECLARE_CYCLE_STAT(TEXT("FPBDRigidsEvolutionGBF::ComputeIntermediateSpatialAcceleration"), STAT_Evolution_ComputeIntermediateSpatialAcceleration, STATGROUP_Chaos);
		DECLARE_CYCLE_STAT(TEXT("FPBDRigidsEvolutionGBF::CreateConstraintGraph"), STAT_Evolution_CreateConstraintGraph, STATGROUP_Chaos);
		DECLARE_CYCLE_STAT(TEXT("FPBDRigidsEvolutionGBF::CreateIslands"), STAT_Evolution_CreateIslands, STATGROUP_Chaos);
		DECLARE_CYCLE_STAT(TEXT("FPBDRigidsEvolutionGBF::PruneCollisions"), STAT_Evolution_PruneCollisions, STATGROUP_Chaos);
		DECLARE_CYCLE_STAT(TEXT("FPBDRigidsEvolutionGBF::AddSleepingContacts"), STAT_Evolution_AddSleepingContacts, STATGROUP_Chaos);
		DECLARE_CYCLE_STAT(TEXT("FPBDRigidsEvolutionGBF::ParallelSolve"), STAT_Evolution_ParallelSolve, STATGROUP_Chaos);
		DECLARE_CYCLE_STAT(TEXT("FPBDRigidsEvolutionGBF::SaveParticlePostSolve"), STAT_Evolution_SavePostSolve, STATGROUP_Chaos);
		DECLARE_CYCLE_STAT(TEXT("FPBDRigidsEvolutionGBF::DeactivateSleep"), STAT_Evolution_DeactivateSleep, STATGROUP_Chaos);
		DECLARE_CYCLE_STAT(TEXT("FPBDRigidsEvolutionGBF::InertiaConditioning"), STAT_Evolution_InertiaConditioning, STATGROUP_Chaos);


		int32 SerializeEvolution = 0;
		FAutoConsoleVariableRef CVarSerializeEvolution(TEXT("p.SerializeEvolution"), SerializeEvolution, TEXT(""));

		bool bChaos_CollisionStore_Enabled = true;
		FAutoConsoleVariableRef CVarCollisionStoreEnabled(TEXT("p.Chaos.CollisionStore.Enabled"), bChaos_CollisionStore_Enabled, TEXT(""));

		// Put the solver into a mode where it reset particles to their initial positions each frame.
		// This is used to test collision detection and - it will be removed
		// NOTE: You should also set the following for dragging in PIE to work while test mode is active:
		// 		p.DisableEditorPhysicsHandle 1
		bool bChaos_Solver_TestMode_Enabled  = false;
		FAutoConsoleVariableRef CVarChaosSolverTestModeEnabled(TEXT("p.Chaos.Solver.TestMode.Enabled"), bChaos_Solver_TestMode_Enabled, TEXT(""));

		bool bChaos_Solver_TestMode_ShowInitialTransforms = false;
		FAutoConsoleVariableRef CVarChaosSolverTestModeShowInitialTransforms(TEXT("p.Chaos.Solver.TestMode.ShowInitialTransforms"), bChaos_Solver_TestMode_ShowInitialTransforms, TEXT(""));
		
		int32 Chaos_Solver_TestMode_Step  = 0;
		FAutoConsoleVariableRef CVarChaosSolverTestModeStep(TEXT("p.Chaos.Solver.TestMode.Step"), Chaos_Solver_TestMode_Step, TEXT(""));
		
		// Set to true to enable some debug validation of the Particle Views every frame
		bool bChaosSolverCheckParticleViews = false;
		FAutoConsoleVariableRef CVarChaosSolverCheckParticleViews(TEXT("p.Chaos.Solver.CheckParticleViews"), bChaosSolverCheckParticleViews, TEXT(""));

		// Enable improved midphase distribution among worker threads. Without this the midphase and narrowphase run on whatever thread the
		// broadphas eoverlap was detected on which tends to give poor distribution.
		bool bChaosMidPhaseRedistributionEnabled = false;
		FAutoConsoleVariableRef CVarChaosSolverMidPhaseRedistributionEnabled(TEXT("p.Chaos.BroadPhase.MidPhaseRedistributionEnabled"), bChaosMidPhaseRedistributionEnabled, TEXT(""));
	}
	using namespace CVars;

	// We want to have largish blocks, but when we are running on multiple cores we don't want to waste too much memory
	int32 CalculateNumCollisionsPerBlock()
	{
		const int32 NumCollisionsPerBlock = 2000;
		const int32 MinCollisionsPerBlock = 100;

		const bool bIsMultithreaded = (FApp::ShouldUseThreadingForPerformance() && !bSingleWorkerPhysics);
		if (!bIsMultithreaded)
		{
			return NumCollisionsPerBlock;
		}
		else
		{
			const int32 NumWorkers = FMath::Min(FTaskGraphInterface::Get().GetNumWorkerThreads(), Chaos::MaxNumWorkers);
			const int32 NumTasks = FMath::Max(NumWorkers, 1);
			return FMath::Max(NumCollisionsPerBlock / NumTasks, MinCollisionsPerBlock);
		}
	}


#if !UE_BUILD_SHIPPING
template <typename TEvolution>
void SerializeToDisk(TEvolution& Evolution)
{
	const TCHAR* FilePrefix = TEXT("ChaosEvolution");
	const FString FullPathPrefix = FPaths::ProfilingDir() / FilePrefix;

	static FCriticalSection CS;	//many evolutions could be running in parallel, serialize one at a time to avoid file conflicts
	FScopeLock Lock(&CS);

	int32 Tries = 0;
	FString UseFileName;
	do
	{
		UseFileName = FString::Printf(TEXT("%s_%d.bin"), *FullPathPrefix, Tries++);
	} while (IFileManager::Get().FileExists(*UseFileName));

	//this is not actually file safe but oh well, very unlikely someone else is trying to create this file at the same time
	TUniquePtr<FArchive> File(IFileManager::Get().CreateFileWriter(*UseFileName));
	if (File)
	{
		FChaosArchive Ar(*File);
		UE_LOGF(LogChaos, Log, "SerializeToDisk File: %ls", *UseFileName);
		Evolution.Serialize(Ar);
	}
	else
	{
		UE_LOGF(LogChaos, Warning, "Could not create file(%ls)", *UseFileName);
	}
}
#endif

// Veryify that no particle is one of the aggregate views more than once. This can happen if
// a particle has been included in more than one orthogonal list (e.g., Active and MovingKinematic)
// - 
template<typename TParticleView>
void CheckParticleViewForDupes(const FString& Name, const TParticleView& ParticleView)
{
	TSet<const FGeometryParticleHandle*> FoundParticles;
	FoundParticles.Reserve(ParticleView.Num());

	for (const auto& Particle : ParticleView)
	{
		const bool bAlreadyInView = FoundParticles.Contains(Particle.Handle());
		if (ensureAlwaysMsgf(!bAlreadyInView, TEXT("%s duplicate Particle <%d>"), *Name, Particle.Handle()->GetHandleIdx()))
		{
			FoundParticles.Add(Particle.Handle());
		}
	}
}

void CheckParticleViewsForDupes(FPBDRigidsSOAs& Particles)
{
	// Check that all particles know what lists they are in
	Particles.CheckListMasks();

	// Check that we have no particles in multiple views
	Particles.CheckViewMasks();

	// A particle appearing twice in either of these results in a race condition because the 
	// collision detection loop will visit the same particle pair twice on different threads.
	CheckParticleViewForDupes(TEXT("NonDisabledDynamicView"), Particles.GetNonDisabledDynamicView());
	CheckParticleViewForDupes(TEXT("ActiveDynamicMovingKinematicParticlesView"), Particles.GetActiveDynamicMovingKinematicParticlesView());

	// No known problems with dupes in these lists, but there should never be dupes
	CheckParticleViewForDupes(TEXT("NonDisabledView"), Particles.GetNonDisabledView());
	CheckParticleViewForDupes(TEXT("NonDisabledClusteredView"), Particles.GetNonDisabledClusteredView());
	CheckParticleViewForDupes(TEXT("ActiveParticlesView"), Particles.GetActiveParticlesView());
	CheckParticleViewForDupes(TEXT("DirtyParticlesView"), Particles.GetDirtyParticlesView());
	CheckParticleViewForDupes(TEXT("AllParticlesView"), Particles.GetAllParticlesView());
	CheckParticleViewForDupes(TEXT("ActiveKinematicParticlesView"), Particles.GetActiveKinematicParticlesView());
	CheckParticleViewForDupes(TEXT("ActiveMovingKinematicParticlesView"), Particles.GetActiveMovingKinematicParticlesView());
	CheckParticleViewForDupes(TEXT("ActiveStaticParticlesView"), Particles.GetActiveStaticParticlesView());
	CheckParticleViewForDupes(TEXT("ActiveDynamicMovingKinematicParticlesView"), Particles.GetActiveDynamicMovingKinematicParticlesView());
}

void CheckMovingKinematicFlag(FPBDRigidsSOAs& Particles)
{
	// Make sure all particles have the correct value for the MovingKinematic flag (NOTE: must be after ApplyKinematicTargets)
	for (const auto& Particle : Particles.GetActiveParticlesView())
	{
		if (const FPBDRigidParticleHandle* Rigid = Particle.Handle()->CastToRigidParticle())
		{
			if (Rigid->IsKinematic())
			{
				const bool bIsMoving = !Rigid->GetV().IsNearlyZero() || !Rigid->GetW().IsNearlyZero();
				ensureMsgf(bIsMoving == Rigid->IsMovingKinematic(),
					TEXT("Kinematic IsMoving flag mismatch. IsMoving=%d V=(%f, %f, %f) W=(%f %f %f) %s"),
					Rigid->IsMovingKinematic(), Rigid->GetV().X, Rigid->GetV().Y, Rigid->GetV().Z, Rigid->GetW().X, Rigid->GetW().Y, Rigid->GetW().Z, *Rigid->GetDebugName());
			}
			else
			{
				ensureMsgf(!Rigid->IsMovingKinematic(), TEXT("Kinematic IsMoving flag set for non-kinematic %s"), *Rigid->GetDebugName());
			}
		}
	}
}

void CheckParticleViewsForErrors(FPBDRigidsSOAs& Particles)
{
	if (CVars::bChaosSolverCheckParticleViews)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_Evolution_CheckParticleViewsForErrors);

		CheckParticleViewsForDupes(Particles);
		CheckMovingKinematicFlag(Particles);
	}
}



void FPBDRigidsEvolutionGBF::Advance(const FReal Dt,const FReal MaxStepDt,const int32 MaxSteps)
{
	// Determine how many steps we would like to take
	int32 NumSteps = FMath::CeilToInt32(Dt / MaxStepDt);
	if (NumSteps > 0)
	{
		PrepareTick();

		// Determine the step time
		const FReal StepDt = Dt / (FReal)NumSteps;

		// Limit the number of steps
		// NOTE: This is after step time calculation so simulation will appear to slow down for large Dt
		// but that is preferable to blowing up from a large timestep.
		NumSteps = FMath::Clamp(NumSteps, 1, MaxSteps);

		for (int32 Step = 0; Step < NumSteps; ++Step)
		{
			// StepFraction: how much of the remaining time this step represents, used to interpolate kinematic targets
			// E.g., for 4 steps this will be: 1/4, 1/2, 3/4, 1
			const FReal StepFraction = (FReal)(Step + 1) / (FReal)(NumSteps);
		
			UE_LOGF(LogChaos, Verbose, "Advance dt = %f [%d/%d]", StepDt, Step + 1, NumSteps);

			AdvanceOneTimeStepImpl(StepDt, FSubStepInfo{ StepFraction, Step, MaxSteps });
		}

		UnprepareTick();
	}
}

void FPBDRigidsEvolutionGBF::AdvanceOneTimeStep(const FReal Dt,const FSubStepInfo& SubStepInfo)
{	
#if CHAOS_DEBUG_DRAW
	// @todo(chaos): will need the time when historical debug draw is supported
	ChaosDD::Private::FChaosDDScopeTimelineContext DDContext(CDDTickTimeline, 0.0f, Dt);
#endif

	PrepareTick();

	AdvanceOneTimeStepImpl(Dt, SubStepInfo);

	UnprepareTick();
}

#if CHAOS_DEBUG_DRAW
void FPBDRigidsEvolutionGBF::SetDebugDrawScene(const ChaosDD::Private::FChaosDDScenePtr& InCDDScene)
{
	CDDScene = InCDDScene;
	CDDTickTimeline.Reset();

	if (CDDScene.IsValid())
	{
		CDDTickTimeline = CDDScene->CreateTimeline(FString::Format(TEXT("{0} {1}"), { CDDScene->GetName(), "Physics Tick" }));
	}
}
#endif

void FPBDRigidsEvolutionGBF::ReloadParticlesCache()
{
	FEvolutionResimCache* ResimCache = GetCurrentStepResimCache();
	if ((ResimCache != nullptr) && ResimCache->IsResimming())
	{
		// Helper function to set particle state
		auto ResimHelper = [this, ResimCache](auto Particle) -> bool
		{
			const bool bParticleAltered = ResimCache->ReloadParticlePostSolve(*Particle->Handle());

			if (bParticleAltered)
			{
				TArray<const Private::FPBDIsland*> Islands = GetIslandManager().FindParticleIslands(Particle->Handle());
				for (const Private::FPBDIsland* Island : Islands)
				{
					if (Island && !Island->IsUsingCache() && !Island->NeedsResim())
					{
						Private::FPBDIsland* IslandNonConst = GetIslandManager().GetIsland(Island->GetArrayIndex());
						IslandNonConst->SetIsUsingCache(true);
					}
				}
			}

			return bParticleAltered;
		};

		auto IterationHelper = [this, ResimHelper](FTransientGeometryParticleHandle& Particle)
		{
			if (Particle.ResimType() == EResimType::FullResim)
			{
				// Don't set state from ResimCache each tick for FullResim particles
				return;
			}

			if (FTransientPBDRigidParticleHandle* Rigid = Particle.CastToRigidParticle())
			{
				if (ResimHelper(Rigid))
				{
					Particles.SetDynamicParticleSOA(Rigid->Handle()); // Update SOA views
				}
			}
			else if (FTransientPBDRigidClusteredParticleHandle* Clustered = Particle.CastToClustered())
			{
				if (ResimHelper(Clustered))
				{
					Particles.SetClusteredParticleSOA(Clustered->Handle()); // Update SOA views
				}
			}
		};

		// Iterate over particles in the different active views
		for (FTransientGeometryParticleHandle& Particle : Particles.GetActiveDynamicMovingKinematicParticlesView())
		{
			IterationHelper(Particle);
		}
		for (FTransientGeometryParticleHandle& Particle : Particles.GetActiveKinematicParticlesView())
		{
			IterationHelper(Particle);
		}
		for (FTransientGeometryParticleHandle& Particle : Particles.GetActiveMovingKinematicParticlesView())
		{
			IterationHelper(Particle);
		}
		for (FTransientGeometryParticleHandle& Particle : Particles.GetActiveParticlesView())
		{
			IterationHelper(Particle);
		}
	}
}

void FPBDRigidsEvolutionGBF::UpdateCollisionSolverType()
{
	// If we have changed the collision solver type we must destroy any existing solvers so that they get recreated.
	// This is not intended to be performant - it is to allow switching for behaviour and performance comparisons only, without restarting the game.
	if (ChaosSolverCollisionSolverType >= 0)
	{
		// Map the cvar to an enum value. Invalid values are assumed to mean Gauss Seidel
		Private::ECollisionSolverType CollisionSolverType = Private::ECollisionSolverType::GaussSeidel;
		if (ChaosSolverCollisionSolverType == 1)
		{
			CollisionSolverType = Private::ECollisionSolverType::GaussSeidelSimd;
		}
		else if (ChaosSolverCollisionSolverType == 2)
		{
			CollisionSolverType = Private::ECollisionSolverType::PartialJacobi;
		}

		if (CollisionSolverType != CollisionConstraints.GetSolverType())
		{
			IslandGroupManager.RemoveConstraintContainer(CollisionConstraints);
			CollisionConstraints.SetSolverType(CollisionSolverType);
			IslandGroupManager.AddConstraintContainer(CollisionConstraints, ChaosSolverCollisionPriority);
		}
	}
}

void FPBDRigidsEvolutionGBF::AdvanceOneTimeStepImpl(const FReal Dt, const FSubStepInfo& SubStepInfo)
{
	CSV_SCOPED_TIMING_STAT(PhysicsVerbose, StepSolver_AdvanceOneTimeStepImpl);
	SCOPE_CYCLE_COUNTER(STAT_Evolution_AdvanceOneTimeStep);

	//for now we never allow solver to schedule more than two tasks back to back
	//this means we only need to keep indices alive for one additional frame
	//the code that pushes indices to pending happens after this check which ensures we won't delete until next frame
	//if sub-stepping is used, the index free will only happen on the first sub-step. However, since we are sub-stepping we would end up releasing half way through interval
	//by checking the step and only releasing on step 0, we ensure the entire interval will see the indices
	if(SubStepInfo.Step == 0)
	{
		Base::ReleasePendingIndices();
	}

#if !UE_BUILD_SHIPPING
	if (SerializeEvolution)
	{
		SerializeToDisk(*this);
	}
#endif

	{
		CVD_SCOPE_TRACE_SOLVER_STEP(CVDDC_EvolutionStart, TEXT("Evolution Start"));
		CVD_TRACE_PARTICLES_SOA(Particles, &Clustering);
	}

	// Update the collision solver type (used to support runtime comparisons of solver types for debugging/testing)
	UpdateCollisionSolverType();

#if CHAOS_EVOLUTION_COLLISION_TESTMODE
	{
		TestModeStep();
		TestModeSaveParticles();
		TestModeRestoreParticles();
		TestModeResetCollisions();
	}
#endif

	
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_Evolution_StepSolver_IntegrateKinematicSpatial);
		CSV_SCOPED_TIMING_STAT(PhysicsVerbose, StepSolver_IntegrateKinematicSpatial);
		if (PreIntegrateCallback != nullptr)
		{
			SCOPE_CYCLE_COUNTER(STAT_Evolution_PreIntegrateCallback);
			PreIntegrateCallback(Dt);
		}

		{
			SCOPE_CYCLE_COUNTER(STAT_Evolution_Integrate);
			CSV_SCOPED_TIMING_STAT(PhysicsVerbose, StepSolver_Integrate);
			Integrate(Dt);
		}

		{
			if (CaptureRewindKinematicTarget)
			{
				CaptureRewindKinematicTarget(Particles.GetActiveMovingKinematicParticlesView());
			}

			SCOPE_CYCLE_COUNTER(STAT_Evolution_KinematicTargets);
			CSV_SCOPED_TIMING_STAT(PhysicsVerbose, StepSolver_KinematicTargets);
			ApplyKinematicTargets(Dt, SubStepInfo.PseudoFraction);
		}

		if (PostIntegrateCallback != nullptr)
		{
			SCOPE_CYCLE_COUNTER(STAT_Evolution_PostIntegrateCallback);
			PostIntegrateCallback(Dt);
		}

		{
			SCOPE_CYCLE_COUNTER(STAT_Evolution_UpdateConstraintPositionBasedState);
			CSV_SCOPED_TIMING_STAT(PhysicsVerbose, StepSolver_UpdateConstraintPositionBasedState);
			UpdateConstraintPositionBasedState(Dt);
		}
		{
			SCOPE_CYCLE_COUNTER(STAT_Evolution_ComputeIntermediateSpatialAcceleration);
			CSV_SCOPED_TIMING_STAT(PhysicsVerbose, StepSolver_ComputeIntermediateSpatialAcceleration);
			Base::ComputeIntermediateSpatialAcceleration();
		}
		TaskDispatcher.WaitTaskEndSpatial(); 
	}

#if WITH_CHAOS_VISUAL_DEBUGGER
		if (FChaosVisualDebuggerTrace::IsTracing())
		{
			WaitIntegrationComplete();
			CVD_SCOPE_TRACE_SOLVER_STEP(CVDDC_PostIntegrate, TEXT("Post Integrate"));
			CVD_TRACE_PARTICLES_SOA(Particles, &Clustering);
		}
#endif

	// Collision detection is sensitive to duplication bugs in the particle views
	// so this is here to help us track them down when they happen
	{
		QUICK_SCOPE_CYCLE_COUNTER(Chaos_CheckParticleViewsForErrors);
		CheckParticleViewsForErrors(Particles);
	}

	{
		SCOPE_CYCLE_COUNTER(STAT_Evolution_DetectCollisions);
		CSV_SCOPED_TIMING_STAT(PhysicsVerbose, StepSolver_DetectCollisions);
		CollisionDetector.GetBroadPhase().SetSpatialAcceleration(InternalAcceleration);

		{
			CVD_SCOPE_TRACE_SOLVER_STEP(CVDDC_CollisionDetectionBroadPhase, TEXT("Collision Detection Broad Phase"));
			QUICK_SCOPE_CYCLE_COUNTER(Chaos_RunBroadPhase);
			CollisionDetector.RunBroadPhase(Dt, GetCurrentStepResimCache());
		}

		if (MidPhaseModifiers)
		{
			SCOPE_CYCLE_COUNTER(STAT_Evolution_MidPhaseModifierCallback);
			ApplyMidPhaseModifier(Dt);
		}

		{
			CVD_SCOPE_TRACE_SOLVER_STEP(CVDDC_CollisionDetectionNarrowPhase, TEXT("Collision Detection Narrow Phase"));
			QUICK_SCOPE_CYCLE_COUNTER(Chaos_RunNarrowPhase);
			CollisionDetector.RunNarrowPhase(Dt, GetCurrentStepResimCache());
		}
	}

	if (PostDetectCollisionsCallback != nullptr)
	{
		SCOPE_CYCLE_COUNTER(STAT_Evolution_PostDetectCollisionsCallback);
		PostDetectCollisionsCallback(Dt);
	}

	{
		SCOPE_CYCLE_COUNTER(STAT_Evolution_TransferJointCollisions);
		TransferJointConstraintCollisions();
	}

	if (CCDModifiers)
	{
		SCOPE_CYCLE_COUNTER(STAT_Evolution_CCDModifierCallback);
		CollisionConstraints.ApplyCCDModifier(*CCDModifiers, Dt);
	}

	if(CollisionModifiers && CVars::bChaosCollisionModiferBeforeCCD)
	{
		SCOPE_CYCLE_COUNTER(STAT_Evolution_CollisionModifierCallback);
		CollisionConstraints.ApplyCollisionModifier(*CollisionModifiers, Dt);
	}

	{
		SCOPE_CYCLE_COUNTER(STAT_Evolution_InertiaConditioning);
		UpdateInertiaConditioning();
	}

	{
		SCOPE_CYCLE_COUNTER(STAT_Evolution_CCD);
		CSV_SCOPED_TIMING_STAT(PhysicsVerbose, CCD);
		CCDManager.ApplyCCD(Dt, CollisionConstraints.GetConstraintAllocator().GetCCDConstraints(), Particles.GetActiveParticlesView().Num());
	}

	if (CollisionModifiers && !CVars::bChaosCollisionModiferBeforeCCD)
	{
		SCOPE_CYCLE_COUNTER(STAT_Evolution_CollisionModifierCallback);
		CollisionConstraints.ApplyCollisionModifier(*CollisionModifiers, Dt);
	}

	{
		SCOPE_CYCLE_COUNTER(STAT_Evolution_CreateConstraintGraph);
		CSV_SCOPED_TIMING_STAT(PhysicsVerbose, StepSolver_CreateConstraintGraph);
		CreateConstraintGraph();
	}
	{
		SCOPE_CYCLE_COUNTER(STAT_Evolution_CreateIslands);
		CSV_SCOPED_TIMING_STAT(PhysicsVerbose, StepSolver_CreateIslands);
		CreateIslands();
	}
	{
		// Once the graph is built (constraints are removed/added) we can destroy any collision that are not required
		SCOPE_CYCLE_COUNTER(STAT_Evolution_PruneCollisions);
		CSV_SCOPED_TIMING_STAT(PhysicsVerbose, StepSolver_PruneCollisions);
		CollisionConstraints.GetConstraintAllocator().PruneExpiredItems();
	}

	if (PreSolveCallback != nullptr)
	{
		SCOPE_CYCLE_COUNTER(STAT_Evolution_PreSolveCallback);
		PreSolveCallback(Dt);
	}

	// todo(chaos) : we are using the main gravity ( index 0 ) we should revise this to account for the various gravities based on the constraint ? 
	CollisionConstraints.SetGravity(GetGravityForces().GetAcceleration(0));

	// Use the cache to update particles in islands that do not require resimming (not desynched)
	{
		CSV_SCOPED_TIMING_STAT(PhysicsVerbose, StepSolver_ReloadCacheTotalSerialized);
		ReloadParticlesCache();
	}

	{
		CVD_SCOPE_TRACE_SOLVER_STEP(CVDDC_PreConstraintSolve, TEXT("Pre Solve"));
		CVD_TRACE_PARTICLES_SOA(Particles, &Clustering);
	}

	// Assign all islands to a set of groups. Each group is solved in parallel with the others.
	{
		SCOPE_CYCLE_COUNTER(STAT_Evolution_BuildGroups);
		CSV_SCOPED_TIMING_STAT(PhysicsVerbose, StepSolver_BuildGroups);

		// If we are resimulating, only build island groups for islands that require to be simulated based of particles sync state and resim type
		IslandGroupManager.BuildGroups(bIsResim);
	}


	TArray<bool> SleepedIslands;
	SleepedIslands.SetNum(GetIslandManager().GetNumIslands());
	TArray<TArray<FPBDRigidParticleHandle*>> DisabledParticles;
	DisabledParticles.SetNum(GetIslandManager().GetNumIslands());
	if(Dt > 0)
	{
		// Solve all the constraints
		{
			SCOPE_CYCLE_COUNTER(STAT_Evolution_ParallelSolve);
			CSV_SCOPED_TIMING_STAT(PhysicsVerbose, StepSolver_PerIslandSolve);

			IslandGroupManager.Solve(Dt);
		}

		// Post-solve CCD fixup to prevent the constraint solve from pushing CCD objects our of the world
		{
			SCOPE_CYCLE_COUNTER(STAT_Evolution_CCDCorrection);
			CSV_SCOPED_TIMING_STAT(PhysicsVerbose, StepSolver_CCDCorrection);
			CCDManager.ProjectCCD(Dt, CollisionConstraints.GetConstraintAllocator().GetCCDConstraints());
		}
	}

	{
		CVD_SCOPE_TRACE_SOLVER_STEP(CVDDC_PostConstraintSolve, TEXT("Post Solve"));
		CVD_TRACE_PARTICLES_SOA(Particles, &Clustering);
	}

	if (PostSolveCallback != nullptr)
	{
		SCOPE_CYCLE_COUNTER(STAT_Evolution_PostSolveCallback);
		PostSolveCallback(Dt);
	}

	{
		SCOPE_CYCLE_COUNTER(STAT_Evolution_SavePostSolve);
		CSV_SCOPED_TIMING_STAT(PhysicsVerbose, StepSolver_SavePostSolve);
		FEvolutionResimCache* ResimCache = GetCurrentStepResimCache();
		if (ResimCache)
		{
			for (const auto& Particle : Particles.GetActiveKinematicParticlesView())
			{
				if (const auto* Rigid = Particle.CastToRigidParticle())
				{
					//NOTE: this assumes the cached values have not changed after the solve (V, W, P, Q should be untouched, otherwise we'll use the wrong values when resim happens)
					ResimCache->SaveParticlePostSolve(*Rigid->Handle());
				}
			}
			for (const auto& Particle : Particles.GetNonDisabledDynamicView())
			{
				//NOTE: this assumes the cached values have not changed after the solve (V, W, P, Q should be untouched, otherwise we'll use the wrong values when resim happens)
				ResimCache->SaveParticlePostSolve(*Particle.Handle());
			}
		}
	}

	{
		SCOPE_CYCLE_COUNTER(STAT_Evolution_DeactivateSleep);
		CSV_SCOPED_TIMING_STAT(PhysicsVerbose, StepSolver_DeactivateSleep);

		// Put any stationary islands to sleep (based on material settings)
		GetIslandManager().UpdateSleep(Dt);
		
		// Disable stationary particles (based on material settings)
		GetIslandManager().UpdateDisable([this](FPBDRigidParticleHandle* Rigid) { DisableParticle(Rigid); });
	}

	Clustering.AdvanceClustering(Dt, GetCollisionConstraints());

	if (CaptureRewindData)
	{
		QUICK_SCOPE_CYCLE_COUNTER(Chaos_CaptureRewindData);
		CaptureRewindData(Particles.GetDirtyParticlesView());
	}

	{
		QUICK_SCOPE_CYCLE_COUNTER(Chaos_ParticleUpdatePosition);
		ParticleUpdatePosition(Particles.GetDirtyParticlesView(), Dt);
	}

	{
		CSV_SCOPED_TIMING_STAT(PhysicsVerbose, StepSolver_IslandEndTick);
		QUICK_SCOPE_CYCLE_COUNTER(Chaos_IslandsEndTick);
		// Clean up the transient data from the constraint graph (e.g., Islands get cleared here)
		GetIslandManager().EndTick();
	}

#if CHAOS_EVOLUTION_COLLISION_TESTMODE
	if (CVars::bChaos_Solver_TestMode_Enabled && CVars::bChaos_Solver_TestMode_ShowInitialTransforms)
	{
		TestModeRestoreParticles();
	}
#endif

	if (CVars::DoFinalProbeNarrowPhase)
	{
		// Run contact updates on probe constraints
		// NOTE: This happens after particles have been moved to their new locations
		// so that we get contacts that are correct for end-of-frame positions
		QUICK_SCOPE_CYCLE_COUNTER(Chaos_DetectProbeCollisions);
		GetCollisionConstraints().DetectProbeCollisions(Dt);
	}
	
	{
		CVD_SCOPE_TRACE_SOLVER_STEP(CVDDC_EvolutionEnd, TEXT("Evolution End"));

		{
			// This needs to be executed withing a CVD Solver step scope,
			// but we are giving it its inner scope so specify a CVD Data channel so this data can be opted out
			// without opting out of the whole step
			CVD_TRACE_STEP_MID_PHASES_FROM_COLLISION_CONSTRAINTS(CVDDC_EndOfEvolutionCollisionConstraints, GetCollisionConstraints());
		}

		CVD_TRACE_JOINT_CONSTRAINTS(CVDDC_JointLinearConstraints, JointConstraints.LinearConstraints);
		CVD_TRACE_JOINT_CONSTRAINTS(CVDDC_JointNonLinearConstraints, JointConstraints.NonLinearConstraints);
		CVD_TRACE_CHARACTER_GROUND_CONSTRAINTS(CVDDC_CharacterGroundConstraints, CharacterGroundConstraints);

		CVD_TRACE_PARTICLES_SOA(Particles, &Clustering);
	}

#if !UE_BUILD_SHIPPING
	if(SerializeEvolution)
	{
		SerializeToDisk(*this);
	}
#endif
}

void FPBDRigidsEvolutionGBF::Integrate(FReal Dt)
{
	CHAOS_SCOPED_TIMER(Integrate);

	const FReal BoundsThickness = GetCollisionConstraints().GetDetectorSettings().BoundsExpansion;
	const FReal VelocityBoundsMultiplier = GetCollisionConstraints().GetDetectorSettings().BoundsVelocityInflation;
	const FReal MaxVelocityBoundsExpansion = GetCollisionConstraints().GetDetectorSettings().MaxVelocityBoundsExpansion;
	const FReal VelocityBoundsMultiplierMACD = GetCollisionConstraints().GetDetectorSettings().BoundsVelocityInflationMACD;
	const FReal MaxVelocityBoundsExpansionMACD = GetCollisionConstraints().GetDetectorSettings().MaxVelocityBoundsExpansionMACD;
	const FReal HackMaxAngularSpeedSq = CVars::HackMaxAngularVelocity * CVars::HackMaxAngularVelocity;
	const FReal HackMaxLinearSpeedSq = CVars::HackMaxVelocity * CVars::HackMaxVelocity;
	const bool bAllowMACD = CVars::bChaosUseMACD;
	const bool bForceMACD = CVars::bChaosForceMACD;
	const bool bIsResimming = IsResimming();
	const bool bIsBubbleResimEnabled = Chaos::ResimConsoleVars::bIsBubbleResimEnabled;

	const TArray<FPBDRigidParticleHandle*>& ParticlesArray = Particles.GetActiveParticlesArray();

	auto IntegrateWork = [this, &ParticlesArray,
		Dt, HackMaxAngularSpeedSq, HackMaxLinearSpeedSq, bAllowMACD, bForceMACD, VelocityBoundsMultiplierMACD, MaxVelocityBoundsExpansionMACD,
		VelocityBoundsMultiplier, MaxVelocityBoundsExpansion, BoundsThickness, bIsResimming, bIsBubbleResimEnabled](int32 BatchIndex, int32 StartIndex, int32 EndIndex)
	{
		for (int32 Index = StartIndex; Index < EndIndex; Index++)
		{
			//question: can we enforce this at the API layer? Right now islands contain non dynamic which makes this hard
			if (FPBDRigidParticleHandle* PBDParticle = ParticlesArray[Index]->CastToRigidParticle(); PBDParticle)
			{
				const bool bIsParticleDynamic = (PBDParticle->ObjectState() == EObjectStateType::Dynamic);
				const bool bIsParticleFrozenDuringBubbleResim = (PBDParticle->ResimType() == EResimType::FrozenDuringResim);
				// We only integrate dynamic particles
				// and when we are resimming, if bubble resim is enabled, we skip integrating particles frozen during resim
				bool bShouldIntegrate = bIsParticleDynamic && !(bIsResimming && bIsBubbleResimEnabled && bIsParticleFrozenDuringBubbleResim);
				if (bShouldIntegrate)
				{
					FPBDRigidParticleHandle& Particle = *PBDParticle;

					FVec3 V = Particle.GetV();
					FVec3 W = Particle.GetW();

					//save off previous velocities
					Particle.SetPreV(V);
					Particle.SetPreW(W);

					for (FForceRule ForceRule : ForceRules)
					{
						ForceRule(Particle, Dt);
					}

					//EulerStepVelocityRule.Apply(Particle, Dt);
					V += Particle.Acceleration() * Dt;
					W += Particle.AngularAcceleration() * Dt;

					//AddImpulsesRule.Apply(Particle, Dt);
					V += Particle.LinearImpulseVelocity();
					W += Particle.AngularImpulseVelocity();
					Particle.LinearImpulseVelocity() = FVec3(0);
					Particle.AngularImpulseVelocity() = FVec3(0);

					//EtherDragRule.Apply(Particle, Dt);

					const FReal LinearDrag = LinearEtherDragOverride >= 0 ? LinearEtherDragOverride : Particle.LinearEtherDrag() * Dt;
					const FReal LinearMultiplier = FMath::Max(FReal(0), FReal(1) - LinearDrag);
					V *= LinearMultiplier;

					const FReal AngularDrag = AngularEtherDragOverride >= 0 ? AngularEtherDragOverride : Particle.AngularEtherDrag() * Dt;
					const FReal AngularMultiplier = FMath::Max(FReal(0), FReal(1) - AngularDrag);
					W *= AngularMultiplier;

					// Apply gyroscopic as a velocity change. This could be applied before the clamps, or after,
					// but the user might be surprised to see an angular velocity that exceeds the clamp
					// so do it before.
					if (Particle.GyroscopicTorqueEnabled())
					{
						W = Utilities::GetAngularVelocityAdjustedForGyroscopicTorques(Particle.RCom(), Particle.I(), W, Dt);
					}

					FReal LinearSpeedSq = V.SizeSquared();
					if (LinearSpeedSq > Particle.MaxLinearSpeedSq())
					{
						V *= FMath::Sqrt(Particle.MaxLinearSpeedSq() / LinearSpeedSq);
					}

					FReal AngularSpeedSq = W.SizeSquared();
					if (AngularSpeedSq > Particle.MaxAngularSpeedSq())
					{
						W *= FMath::Sqrt(Particle.MaxAngularSpeedSq() / AngularSpeedSq);
					}

					if (CVars::HackMaxAngularVelocity >= 0.f)
					{
						AngularSpeedSq = W.SizeSquared();
						if (AngularSpeedSq > HackMaxAngularSpeedSq)
						{
							W = W * (CVars::HackMaxAngularVelocity / FMath::Sqrt(AngularSpeedSq));
						}
					}

					if (CVars::HackMaxVelocity >= 0.f)
					{
						LinearSpeedSq = V.SizeSquared();
						if (LinearSpeedSq > HackMaxLinearSpeedSq)
						{
							V = V * (CVars::HackMaxVelocity / FMath::Sqrt(LinearSpeedSq));
						}
					}

					FVec3 PCoM = Particle.XCom();
					FRotation3 QCoM = Particle.RCom();

					PCoM = PCoM + V * Dt;
					QCoM = FRotation3::IntegrateRotationWithAngularVelocity(QCoM, W, Dt);

					Particle.SetTransformPQCom(PCoM, QCoM);
					Particle.SetV(V);
					Particle.SetW(W);

					// We need to expand the bounds back along velocity otherwise we can miss collisions when a box
					// lands on another box, imparting velocity to the lower box and causing the boxes to be
					// separated by more than Cull Distance at collision detection time.
					// NOTE: We use a different (larger) max bounds expansion when MACD is enabled.
					FVec3 VelocityBoundsDelta = FVec3(0);
					const bool bIsMACD = bForceMACD || (bAllowMACD && Particle.MACDEnabled());
					const FReal ParticleVelocityBoundsMultiplier = bIsMACD ? VelocityBoundsMultiplierMACD : VelocityBoundsMultiplier;
					const FReal ParticleMaxVelocityBoundsExpansion = bIsMACD ? MaxVelocityBoundsExpansionMACD : MaxVelocityBoundsExpansion;
					if ((ParticleVelocityBoundsMultiplier > 0) && (ParticleMaxVelocityBoundsExpansion > 0))
					{
						VelocityBoundsDelta = (-ParticleVelocityBoundsMultiplier * Dt) * V;
						VelocityBoundsDelta = VelocityBoundsDelta.BoundToCube(ParticleMaxVelocityBoundsExpansion);
					}

					if (!Particle.CCDEnabled())
					{
						// Expand bounds about P/Q by a small amount. This can still result in missed collisions, especially
						// when we have joints that pull the body back to X/R, if P-X is greater than the BoundsThickness
						Particle.UpdateWorldSpaceStateSwept(FRigidTransform3(Particle.GetP(), Particle.GetQ()), FVec3(BoundsThickness), VelocityBoundsDelta);
					}
					else
					{

#if CHAOS_DEBUG_DRAW
						if (CVars::ChaosSolverDrawCCDThresholds)
						{
							DebugDraw::DrawCCDAxisThreshold(Particle.GetX(), Particle.CCDAxisThreshold(), Particle.GetP() - Particle.GetX(), Particle.GetQ());
						}
#endif

						if (FCCDHelpers::DeltaExceedsThreshold(Particle.CCDAxisThreshold(), Particle.GetP() - Particle.GetX(), Particle.GetQ()))
						{
							// We sweep the bounds from P back along the velocity and expand by a small amount.
							// If not using tight bounds we also expand the bounds in all directions by Velocity. This is necessary only for secondary CCD collisions
							// @todo(chaos): expanding the bounds by velocity is very expensive - revisit this
							const FVec3 VDt = V * Dt;
							FReal CCDBoundsExpansion = BoundsThickness;
							if (!CVars::bChaosCollisionCCDUseTightBoundingBox && (CVars::ChaosCollisionCCDConstraintMaxProcessCount > 1))
							{
								CCDBoundsExpansion += VDt.GetAbsMax();
							}
							Particle.UpdateWorldSpaceStateSwept(FRigidTransform3(Particle.GetP(), Particle.GetQ()), FVec3(CCDBoundsExpansion), -VDt);
						}
						else
						{
							Particle.UpdateWorldSpaceStateSwept(FRigidTransform3(Particle.GetP(), Particle.GetQ()), FVec3(BoundsThickness), VelocityBoundsDelta);
						}
					}
				}

				if (!Base::IsADisableCluster(*PBDParticle) && (PBDParticle->HasCollision() || ForceNoCollisionIntoSQ))
				{
					const FUniqueIdx UniqueIdx = PBDParticle->UniqueIdx();
					FPendingSpatialData& PendingData = InternalAccelerationQueue.Add(BatchIndex + 1, UniqueIdx);
					PendingData.Operation = EPendingSpatialDataOperation::Update;
					PendingData.AccelerationHandle = FAccelerationStructureHandle(*PBDParticle);
					PendingData.SpatialIdx = PBDParticle->SpatialIdx();
				}
			}
		}
	};

	auto DirtyAsync = [this, &ParticlesArray]()
	{
		// The async data structure is for the static particle. Here we update the dynamic particles.
		// We need to update them only if dynamic and static particles are sharing the same spatial structure
		// If the dynamic tree is not used we need to update the static one.
		if (AccelerationStructureSplitStaticAndDynamic != 1 || AccelerationStructureUseDynamicTree == 0)
		{
			for (FPBDRigidParticleHandle* Particle : ParticlesArray)
			{
				Base::DirtyParticleAsync(*Particle);
			}
		}
	};
	

	if (!Chaos::ShouldExecuteTasks(Particles.GetActiveDynamicMovingKinematicParticlesView().Num()))
	{
		// In single threaded mode keep same logic with three batches inside PendingDataArrays: 
		// 0 Dirty particles from the PushData
		// 1 Dynamic particles added in Integrate
		// 2 Kinematic particles added in Apply Kinematic Target
		InternalAccelerationQueue.PendingDataArrays.SetNum(3);
		InternalAccelerationQueue.KinematicBatchStartIndex = 2;
		const int32 NumDynParticles = ParticlesArray.Num();
		IntegrateWork(0, 0, NumDynParticles);
		DirtyAsync();
	}
	else
	{
		TaskDispatcher.DispatchIntegrate(IntegrateWork);
		TaskDispatcher.DispatchDynAsyncDirty(DirtyAsync);
	}
}

void FPBDRigidsEvolutionGBF::ApplyKinematicTargets(const FReal Dt, const FReal StepFraction)
{
	check(StepFraction > (FReal)0);
	check(StepFraction <= (FReal)1);

	const bool bIsLastStep = (FMath::IsNearlyEqual(StepFraction, (FReal)1, (FReal)UE_KINDA_SMALL_NUMBER));

	TParticleView<FPBDRigidParticles>& KienamticParticles = Particles.GetActiveMovingKinematicParticlesView();
	using TSOA = typename TParticleView<FPBDRigidParticles>::TSOA;
	using THandle = typename TSOA::THandleType;
	using THandleBase = typename THandle::THandleBase;
	using TTransientHandle = typename THandle::TTransientHandle;
	int32 DispatchBatchIndex = 0;
	for (int32 ViewIndex = 0; ViewIndex < KienamticParticles.SOAViews.Num(); ++ViewIndex)
	{
		const TSOAView<TSOA>& SOAView = KienamticParticles.SOAViews[ViewIndex];
		const int32 NumParticles = SOAView.Size();
		if (NumParticles == 0)
		{
			continue;
		}
		// Dispatch tasks according if it is an array of handles or a SOA
		if (const TArray<THandle*>* CurHandlesArray = SOAView.HandlesArray)
		{
			auto ReturnHandle = [CurHandlesArray](int32 Index) -> TParticleView<FPBDRigidParticles>::THandle::THandleBase
				{
					THandle* HandlePtr = (*CurHandlesArray)[Index];
					return THandleBase(HandlePtr->GeometryParticles, HandlePtr->ParticleIdx);
				};
			using LambdaType = decltype(ReturnHandle);
			const int32 NumHandles = CurHandlesArray->Num();
			DispatchKinematicTargetsTasks<LambdaType>(DispatchBatchIndex++, NumHandles, Dt, StepFraction, bIsLastStep, ReturnHandle);
		}
		else if (TSOA* Soa = SOAView.SOA)
		{
			auto ReturnHandle = [Soa](int32 Index) { return THandleBase(Soa, Index); };
			using LambdaType = decltype(ReturnHandle);
			DispatchKinematicTargetsTasks<LambdaType>(DispatchBatchIndex++, NumParticles, Dt, StepFraction, bIsLastStep, ReturnHandle);
		}
	}

	auto DirtyKinAsync = [this]()
	{
		// The async data structure is for the static particle. Here we update the kinematic particles.
		// We need to update them only if kinematic and static particles are sharing the same spatial structure
		// If the dynamic tree is not used we need to update the static one.
		if (AccelerationStructureSplitStaticAndDynamic != 1 || AccelerationStructureUseDynamicTree == 0)
		{
			for (FTransientPBDRigidParticleHandle& Particle : Particles.GetActiveMovingKinematicParticlesView())
			{
				// Moving kinematics need to have SQ updates queued to apply substepped movement to the
				// physics thread acceleration structure in order to get correct kinematic collisions
				// and pass a correct structure back to the external thread
				DirtyParticleAsync(Particle);
			}
		}
	};

	// At this stage we should be sure if we have launched tasks to linked them
	// or if we are running on a single thread
	if (!Chaos::ShouldExecuteTasks(Particles.GetActiveDynamicMovingKinematicParticlesView().Num()))
	{
		DirtyKinAsync();
		// done with update, let's clear the tracking structures
		if (bIsLastStep)
		{
			constexpr bool bUpdateView = false;
			Particles.UpdateAllMovingKinematic(bUpdateView);
		}
		Particles.UpdateDirtyViews();
	}
	else
	{
		TaskDispatcher.DispatchKinAsyncDirtyAndUpdateKinematic(DirtyKinAsync, bIsLastStep);
	}
}

void FPBDRigidsEvolutionGBF::WaitIntegrationComplete()
{
	TaskDispatcher.WaitIntegrationComplete();
}

template <typename LambdaType>
void FPBDRigidsEvolutionGBF::DispatchKinematicTargetsTasks(int32 DispatchBatchIndex, int32 NumParticles, FReal Dt, FReal StepFraction, bool bIsLastStep, LambdaType ReturnHandle)
{
	using TSOA = typename TParticleView<FPBDRigidParticles>::TSOA;
	using THandle = typename TSOA::THandleType;
	using THandleBase = typename THandle::THandleBase;

	auto KinematicTargetsLambda = [this, ReturnHandle, Dt, StepFraction, bIsLastStep](int32 StartIndex, int32 EndIndex, int32 BatchIndex)
	{
		InternalAccelerationQueue.PendingDataArrays[InternalAccelerationQueue.KinematicBatchStartIndex + BatchIndex].Reserve(EndIndex- StartIndex);
		for (int32 Index = StartIndex; Index < EndIndex; Index++)
		{
			THandleBase Handle = ReturnHandle(Index);
			FTransientPBDRigidParticleHandle& Particle = static_cast<FTransientPBDRigidParticleHandle&>(Handle);
			if (!Particle.LightWeightDisabled())
			{

				FKinematicTarget& KinematicTarget = Particle.KinematicTarget();
				const FVec3 CurrentX = Particle.GetX();
				const FRotation3f CurrentR = Particle.GetRf();
				constexpr FReal MinDt = 1e-6f;

				bool bMoved = false;
				switch (KinematicTarget.GetMode())
				{
				case EKinematicTargetMode::None:
					// Nothing to do
					break;

				case EKinematicTargetMode::Reset:
				{
					// Reset velocity and then switch to do-nothing mode
					Particle.SetVf(FVec3f(0.0f, 0.0f, 0.0f));
					Particle.SetWf(FVec3f(0.0f, 0.0f, 0.0f));
					Particle.ClearIsMovingKinematic();
					KinematicTarget.SetMode(EKinematicTargetMode::None);
					break;
				}

				case EKinematicTargetMode::Position:
				{
					// Move to kinematic target and update velocities to match
					// Target positions only need to be processed once, and we reset the velocity next frame (if no new target is set)
					FVec3 NewX;
					FRotation3f NewR;
					if (bIsLastStep)
					{
						NewX = KinematicTarget.GetPosition();
						NewR = KinematicTarget.GetRotation();
						KinematicTarget.SetMode(EKinematicTargetMode::Reset);
					}
					else
					{
						// as a reminder, stepfraction is the remaing fraction of the step from the remaining steps
						// for total of 4 steps and current step of 2, this will be 1/3 ( 1 step passed, 3 steps remains )
						NewX = FVec3::Lerp(CurrentX, KinematicTarget.GetPosition(), StepFraction);
						NewR = FRotation3f::Slerp(CurrentR, KinematicTarget.GetRotation(), decltype(FQuat4f::X)(StepFraction));
					}

					const bool bPositionChanged = !FVec3::IsNearlyEqual(NewX, CurrentX, UE_SMALL_NUMBER);
					const bool bRotationChanged = !FRotation3::IsNearlyEqual(NewR, CurrentR, UE_SMALL_NUMBER);
					bMoved = bPositionChanged || bRotationChanged;
					FVec3 NewV = FVec3(0);
					FVec3f NewW = FVec3f(0);
					if (Dt > MinDt)
					{
						if (bPositionChanged)
						{
							NewV = FVec3::CalculateVelocity(CurrentX, NewX, Dt);
						}
						if (bRotationChanged)
						{
							NewW = FRotation3f::CalculateAngularVelocity(CurrentR, NewR, static_cast<FRealSingle>(Dt));
						}
					}
					Particle.SetX(NewX);
					Particle.SetRf(NewR);
					Particle.SetV(NewV);
					Particle.SetWf(NewW);
					Particle.SetIsMovingKinematic();

					break;
				}

				case EKinematicTargetMode::Velocity:
				{
					// Move based on velocity
					bMoved = true;
					Particle.SetX(Particle.GetX() + Particle.GetV() * Dt);
					Particle.SetRf(FRotation3f::IntegrateRotationWithAngularVelocity(Particle.GetRf(), Particle.GetWf(), FRealSingle(Dt)));
					Particle.SetIsMovingKinematic();

					break;
				}
				}

				// Set positions and previous velocities if we can
				// Note: At present kinematics are in fact rigid bodies
				Particle.SetP(Particle.GetX());
				Particle.SetQf(Particle.GetRf());
				Particle.SetPreVf(Particle.GetVf());
				Particle.SetPreWf(Particle.GetWf());

				if (bMoved)
				{
					if (!Particle.CCDEnabled())
					{
						Particle.UpdateWorldSpaceState(FRigidTransform3(Particle.GetP(), Particle.GetQ()), FVec3(0));
					}
					else
					{
						Particle.UpdateWorldSpaceStateSwept(FRigidTransform3(Particle.GetP(), Particle.GetQ()), FVec3(0), -Particle.GetV() * Dt);
					}
				}
			}
			if (!Base::IsADisableCluster(Particle) && (Particle.HasCollision() || ForceNoCollisionIntoSQ))
			{
				const FUniqueIdx UniqueIdx = Particle.UniqueIdx();
				FPendingSpatialData& PendingData = InternalAccelerationQueue.Add(InternalAccelerationQueue.KinematicBatchStartIndex + BatchIndex, UniqueIdx);
				PendingData.Operation = EPendingSpatialDataOperation::Update;
				PendingData.AccelerationHandle = FAccelerationStructureHandle(Particle);
				PendingData.SpatialIdx = Particle.SpatialIdx();
			}
		}
	};

	if (!Chaos::ShouldExecuteTasks(Particles.GetActiveDynamicMovingKinematicParticlesView().Num()))
	{
		KinematicTargetsLambda(0, NumParticles, 0);
	}
	else
	{
		TaskDispatcher.DispatchKinematicsTarget(KinematicTargetsLambda, NumParticles, DispatchBatchIndex); 
	}
}

void FPBDRigidsEvolutionGBF::ApplyMidPhaseModifier(FReal Dt)
{
	if (MidPhaseModifiers != nullptr && !MidPhaseModifiers->IsEmpty())
	{
		UE::Tasks::Wait(CollisionDetector.GetBroadPhase().GetBroadToNarrowTasks());

		// Consolidate midphases from parallel threads into the main lists held by the allocator.
		// This makes them accessible to MidPhaseModifiers.
		CollisionConstraints.GetConstraintAllocator().ProcessNewMidPhases();

		FMidPhaseModifierAccessor MidPhaseModifier(*this);
		for (ISimCallbackObject* ModifierCallback : *MidPhaseModifiers)
		{
			ModifierCallback->MidPhaseModification_Internal(MidPhaseModifier);
		}
	}
}

void FPBDRigidsEvolutionGBF::SetIsDeterministic(const bool bInIsDeterministic)
{
	// We detect collisions in parallel, so order is non-deterministic without additional processing
	CollisionConstraints.SetIsDeterministic(bInIsDeterministic);

	// IslandManager uses TSparseArray which requires free-list maintenance for determinism
	IslandManager.SetIsDeterministic(bInIsDeterministic);
}

void FPBDRigidsEvolutionGBF::SetShockPropagationIterations(const int32 InPositionIts, const int32 InVelocityIts)
{
	// Negative inputs mean leave values as they are
	if (InPositionIts >= 0)
	{
		CollisionConstraints.SetPositionShockPropagationIterations(InPositionIts);
	}

	if (InVelocityIts >= 0)
	{
		CollisionConstraints.SetVelocityShockPropagationIterations(InVelocityIts);
	}

	// If we have shock prop enabled, we require levels to be assigned to constraints and bodies
	IslandManager.SetAssignLevels(CollisionConstraints.IsShockPropagationEnabled());
}


void FPBDRigidsEvolutionGBF::ResetCollisions()
{
	for (FPBDCollisionConstraintHandle* Collision : CollisionConstraints.GetConstraintHandles())
	{
		if (Collision != nullptr)
		{
			Collision->GetContact().ResetSavedManifoldPoints();
			Collision->GetContact().GetGJKWarmStartData().Reset();
		}
	}
}

FPBDRigidsEvolutionGBF::FPBDRigidsEvolutionGBF(
	FPBDRigidsSOAs& InParticles, 
	THandleArray<FChaosPhysicsMaterial>& SolverPhysicsMaterials, 
	const TArray<ISimCallbackObject*>* InMidPhaseModifiers,
	const TArray<ISimCallbackObject*>* InCCDModifiers,
	const TArray<ISimCallbackObject*>* InStrainModifiers,
	const TArray<ISimCallbackObject*>* InCollisionModifiers,
	bool InIsSingleThreaded)
	: Base(InParticles, SolverPhysicsMaterials, InIsSingleThreaded)
	, Clustering(*this, Particles.GetClusteredParticles(), InStrainModifiers)
	, CollisionConstraints(InParticles, Collided, PhysicsMaterials, PerParticlePhysicsMaterials, &SolverPhysicsMaterials, CalculateNumCollisionsPerBlock(), DefaultRestitutionThreshold)
	, BroadPhase(InParticles)
	, CollisionDetector(BroadPhase, CollisionConstraints)
	, PreIntegrateCallback(nullptr)
	, PostIntegrateCallback(nullptr)
	, PreSolveCallback(nullptr)
	, PostSolveCallback(nullptr)
	, CurrentStepResimCacheImp(nullptr)
	, MidPhaseModifiers(InMidPhaseModifiers)
	, CCDModifiers(InCCDModifiers)
	, CollisionModifiers(InCollisionModifiers)
	, CCDManager()
	, bIsDeterministic(false)
	, InertiaConditioningEpoch(-1)
{
	SetNumPositionIterations(DefaultNumPositionIterations);
	SetNumVelocityIterations(DefaultNumVelocityIterations);
	SetNumProjectionIterations(DefaultNumProjectionIterations);

	CollisionConstraints.SetCanDisableContacts(!!CollisionDisableCulledContacts);

	CollisionConstraints.SetCullDistance(DefaultCollisionCullDistance);

	JointConstraints.LinearConstraints.SetUseSimd(bChaosRigids_UseSimdForJointsSolver); //NonLinear Joints don't have simd code so it's skipped. 

	GetIslandManager().SetMaterialContainers(&PhysicsMaterials, &PerParticlePhysicsMaterials, &SolverPhysicsMaterials);
	GetIslandManager().SetGravityForces(&GravityForces);
	GetIslandManager().SetDisableCounterThreshold(DisableThreshold);

	SetParticleUpdatePositionFunction([this](const TParticleView<FPBDRigidParticles>& ParticlesInput, const FReal Dt)
	{
		ParticlesInput.ParallelFor([&](auto& Particle, int32 Index)
		{
			Particle.SetX(Particle.GetP());
			Particle.SetR(Particle.GetQ());

			//TODO: rename this function since it's not just updating position
			Particle.SetPreObjectStateLowLevel(Particle.ObjectState());
		});
	});

	AddForceFunction([this](TTransientPBDRigidParticleHandle<FReal, 3>& HandleIn, const FReal Dt)
	{
		GravityForces.Apply(HandleIn, Dt);
	});

	AddConstraintContainer(SuspensionConstraints, ChaosSolverSuspensionPriority);
	AddConstraintContainer(CollisionConstraints, ChaosSolverCollisionPriority);
	AddConstraintContainer(JointConstraints.LinearConstraints, ChaosSolverJointPriority);
	AddConstraintContainer(JointConstraints.NonLinearConstraints, ChaosSolverJointPriority);
	AddConstraintContainer(CharacterGroundConstraints, ChaosSolverCharacterGroundConstraintPriority);

	SetInternalParticleInitilizationFunction([](const FGeometryParticleHandle*, const FGeometryParticleHandle*) {});
}

FPBDRigidsEvolutionGBF::~FPBDRigidsEvolutionGBF()
{
	// This is really only needed to ensure proper cleanup (we verify that constraints have been removed from 
	// the graph in the destructor). This can be optimized if it's a problem but it shouldn't be
	GetIslandManager().Reset();
}

void FPBDRigidsEvolutionGBF::Serialize(FChaosArchive& Ar)
{
	Base::Serialize(Ar);
}

TUniquePtr<IResimCacheBase> FPBDRigidsEvolutionGBF::CreateExternalResimCache() const
{
	return TUniquePtr<IResimCacheBase>(new FEvolutionResimCache());
}

void FPBDRigidsEvolutionGBF::SetCurrentStepResimCache(IResimCacheBase* InCurrentStepResimCache)
{
	CurrentStepResimCacheImp = static_cast<FEvolutionResimCache*>(InCurrentStepResimCache);
}

FPBDJointConstraintHandle* FPBDRigidsEvolutionGBF::CreateJointConstraint(const FParticlePair& InConstrainedParticles, const FPBDJointSettings& InJointSettings)
{
	FPBDJointConstraintHandle* JointHandle = nullptr;

	if (InJointSettings.bUseLinearSolver)
	{
		JointHandle = JointConstraints.LinearConstraints.AddConstraint(InConstrainedParticles, InJointSettings.ConnectorTransforms);
	}
	else
	{
		JointHandle = JointConstraints.NonLinearConstraints.AddConstraint(InConstrainedParticles, InJointSettings.ConnectorTransforms);
	}

	if (!InJointSettings.bCollisionEnabled)
	{
		GetBroadPhase().GetIgnoreCollisionManager().AddIgnoreCollisions(InConstrainedParticles[0], InConstrainedParticles[1]);
	}

	JointHandle->SetSettings(InJointSettings);

	// Register the constraint with each particle
	InConstrainedParticles[0]->AddConstraintHandle(JointHandle);
	InConstrainedParticles[1]->AddConstraintHandle(JointHandle);

	// We added a joint to the particles, so we need to modify the inertia (See UpdateInertiaConditioning)
	FGenericParticleHandle(InConstrainedParticles[0])->SetInertiaConditioningDirty();
	FGenericParticleHandle(InConstrainedParticles[1])->SetInertiaConditioningDirty();

	return JointHandle;
}

void FPBDRigidsEvolutionGBF::DestroyJointConstraint(FPBDJointConstraintHandle* InJointHandle)
{
	FParticlePair ConstrainedParticles = InJointHandle->GetConstrainedParticles();

	const FPBDJointSettings& Settings = InJointHandle->GetSettings();
	const bool bValidPair = ConstrainedParticles[0] && ConstrainedParticles[1];

	// If this constraint disables collisions - need to restore the collisions after we destroy the constraint
	if (bValidPair && !Settings.bCollisionEnabled)
	{
		GetBroadPhase().GetIgnoreCollisionManager().RemoveIgnoreCollisions(ConstrainedParticles[0], ConstrainedParticles[1]);
	}

	// Unegister the constraint from each particle
	if (ConstrainedParticles[0])
	{
		ConstrainedParticles[0]->RemoveConstraintHandle(InJointHandle);
	}
	if (ConstrainedParticles[1])
	{
		ConstrainedParticles[1]->RemoveConstraintHandle(InJointHandle);
	}

	// Remove from the constraint graph
	if (InJointHandle->IsInConstraintGraph())
	{
		RemoveConstraintFromConstraintGraph(InJointHandle);
	}

	// Destroy the constraint
	if (Settings.bUseLinearSolver)
	{
		JointConstraints.LinearConstraints.RemoveConstraint(InJointHandle->GetConstraintIndex());
	}
	else
	{
		JointConstraints.NonLinearConstraints.RemoveConstraint(InJointHandle->GetConstraintIndex());
	}
}

FPBDJointConstraintHandle* FPBDRigidsEvolutionGBF::SetJointConstraintSettings(FPBDJointConstraintHandle* InJointHandle, const FPBDJointSettings& InJointSettings)
{
	const FPBDJointSettings& CurrentJointSettings = InJointHandle->GetSettings();
	FParticlePair ConstrainedParticles = InJointHandle->GetConstrainedParticles();

	// Handle changes to the CollisionEnabled flag
	if (CurrentJointSettings.bCollisionEnabled != InJointSettings.bCollisionEnabled)
	{
		if (InJointSettings.bCollisionEnabled)
		{
			GetBroadPhase().GetIgnoreCollisionManager().RemoveIgnoreCollisions(ConstrainedParticles[0], ConstrainedParticles[1]);
		}
		else
		{
			GetBroadPhase().GetIgnoreCollisionManager().AddIgnoreCollisions(ConstrainedParticles[0], ConstrainedParticles[1]);
		}
	}

	// NOTE: Changing the solver type requires us to destroy and recreate the joint, which results in
	// relocating the joint handle. This also resets any state in the joint, e.g., whether it is broken.
	// @todo(chaos): Improve suport for this feature, but for now, just make it not crash!
	if (CurrentJointSettings.bUseLinearSolver != InJointSettings.bUseLinearSolver)
	{
		DestroyJointConstraint(InJointHandle);
		return CreateJointConstraint(ConstrainedParticles, InJointSettings);
	}

	// Update the joint settings
	InJointHandle->SetSettings(InJointSettings);
	return InJointHandle;
}

void FPBDRigidsEvolutionGBF::TransferJointConstraintCollisions()
{
	//
	// Stubbed out implementation of joint constraint collision transfer. 
	// Currently disabled because its not required, and the implementation
	// requires proper unit testing before it should be released. 
	//
#if 0
	// Transfer collisions from the child of a joint to the parent.
	// E.g., if body A and B are connected by a joint, with A the parent and B the child...
	// then a third body C collides with B...
	// we create a new collision between A and C at the same world position.
	// E.g., This can be used to forward collision impulses from a vehicle bumper to its
	// chassis without having to worry about making the joint connecting them very stiff
	// which is quite difficult for large mass ratios and would require many iterations.
	if (DoTransferJointConstraintCollisions)
	{
		Private::FCollisionConstraintAllocator& CollisionAllocator = CollisionConstraints.GetConstraintAllocator();

		// @todo(chaos): we should only visit the joints that have ContactTransferScale > 0 
		for (int32 JointConstraintIndex = 0; JointConstraintIndex < GetJointConstraints().NumConstraints(); ++JointConstraintIndex)
		{
			FPBDJointConstraintHandle* JointConstraint = GetJointConstraints().GetConstraintHandle(JointConstraintIndex);
			const FPBDJointSettings& JointSettings = JointConstraint->GetSettings();
			if (JointSettings.ContactTransferScale > FReal(0))
			{
				FGenericParticleHandle ParentParticle = JointConstraint->GetConstrainedParticles()[1];
				FGenericParticleHandle ChildParticle = JointConstraint->GetConstrainedParticles()[0];

				const FRigidTransform3 ParentTransform = FParticleUtilities::GetActorWorldTransform(ParentParticle);
				const FRigidTransform3 ChildTransform = FParticleUtilities::GetActorWorldTransform(ChildParticle);
				const FRigidTransform3 ChildToParentTransform = ChildTransform.GetRelativeTransform(ParentTransform);

				ChildParticle->Handle()->ParticleCollisions().VisitCollisions(
					[&](const FPBDCollisionConstraint* VisitedConstraint)
					{
						if (VisitedConstraint->GetCCDEnabled())
						{
							return;
						}

						// @todo(chaos): implemeent this
						// Note: the defined out version has a couple issues we will need to address in the new version
						//	-	it passes Implicit pointers from one body to a constraint whose lifetime is not controlled by that body
						//		which could cause problems if the first body is destroyed.
						//	-	we need to properly support collisions constraints without one (or both) Implicit Objects. Collisions are 
						//		managed per shape pair, and found by a key that depends on them, so we'd need to rethink that a bit. 
						//		Here it's useful to be able to use the child implicit to generate the unique key, but we don't want the 
						//		constraint to hold the pointer (see previous issue).
						//	-	we should check to see if there is already an active constraint between the bodies because we don't want
						//		to replace a legit collision with our fake one...probably
						//const FGeometryParticleHandle* NewParentParticleConst = (CurrConstraint->GetParticle0() == ChildParticle->Handle()) ? CurrConstraint->GetParticle1() : ChildCollisionConstraint->GetParticle0();
						//FGeometryParticleHandle* NewParticleA = const_cast<FGeometryParticleHandle*>(NewParentParticleConst);
						//FGeometryParticleHandle* NewParticleB = ParentParticle->Handle();

						FPBDCollisionConstraint& CollisionConstraint = *const_cast<FPBDCollisionConstraint*>(VisitedConstraint);
						// If the collision constraint is colliding with the child of the joint, then 
						// we want to map the collision between the parent of the joint and the external
						// collision body.Otherwise, we map the collision between the childand 
						// the external collision body.The new collision constraint is constructed with
						// the implicit object of the colliding body on the joint, to the particle the
						// collision was transferred to.

						auto MakeCollisionAtIndex0 = [&ChildToParentTransform, &CollisionConstraint](FGenericParticleHandle& TransferToParticle)
						{
							return FPBDCollisionConstraint::Make(
								TransferToParticle->Handle(),
								CollisionConstraint.GetImplicit0(),
								CollisionConstraint.GetCollisionParticles0(),
								ChildToParentTransform.GetRelativeTransform(CollisionConstraint.GetShapeRelativeTransform0()),
								CollisionConstraint.GetParticle1(),
								CollisionConstraint.GetImplicit1(),
								CollisionConstraint.GetCollisionParticles1(),
								CollisionConstraint.GetShapeRelativeTransform1(),
								CollisionConstraint.GetCullDistance(),
								CollisionConstraint.GetUseManifold(),
								CollisionConstraint.GetShapesType()
							);
						};

						auto MakeCollisionAtIndex1 = [&ChildToParentTransform, &CollisionConstraint](FGenericParticleHandle& TransferToParticle)
						{
							return FPBDCollisionConstraint::Make(
								CollisionConstraint.GetParticle1(),
								CollisionConstraint.GetImplicit1(),
								CollisionConstraint.GetCollisionParticles1(),
								CollisionConstraint.GetShapeRelativeTransform1(),
								TransferToParticle->Handle(),
								CollisionConstraint.GetImplicit0(),
								CollisionConstraint.GetCollisionParticles0(),
								ChildToParentTransform.GetRelativeTransform(CollisionConstraint.GetShapeRelativeTransform1()),
								CollisionConstraint.GetCullDistance(),
								CollisionConstraint.GetUseManifold(),
								CollisionConstraint.GetShapesType()
							);
						};

						FGenericParticleHandle CollisionParticleA, CollisionParticleB;
						TUniquePtr<FPBDCollisionConstraint> TransferedConstraint;
						if (CollisionConstraint.GetParticle0() == ChildParticle->CastToRigidParticle() ||
							CollisionConstraint.GetParticle1() == ChildParticle->CastToRigidParticle())
						{
							if (CollisionConstraint.GetParticle0() == ChildParticle->CastToRigidParticle())
							{
								CollisionParticleA = ParentParticle;
								CollisionParticleB = CollisionConstraint.GetParticle1();
								TransferedConstraint = MakeCollisionAtIndex0(ParentParticle);
							}
							else
							{
								CollisionParticleA = CollisionConstraint.GetParticle0();
								CollisionParticleB = ParentParticle;
								TransferedConstraint = MakeCollisionAtIndex1(ParentParticle);
							}
						}
						else if (CollisionConstraint.GetParticle0() == ParentParticle->CastToRigidParticle() ||
							CollisionConstraint.GetParticle1() == ParentParticle->CastToRigidParticle())
						{
							if (CollisionConstraint.GetParticle0() == ParentParticle->CastToRigidParticle())
							{
								CollisionParticleA = ChildParticle;
								CollisionParticleB = CollisionConstraint.GetParticle0();
								TransferedConstraint = MakeCollisionAtIndex0(ChildParticle);
							}
							else
							{
								CollisionParticleA = CollisionConstraint.GetParticle1();
								CollisionParticleB = ChildParticle;
								TransferedConstraint = MakeCollisionAtIndex1(ChildParticle);
							}
						}

						TArray<FManifoldPoint> TransferManifolds;
						for (FManifoldPoint& Manifold : CollisionConstraint.GetManifoldPoints())
						{
							TransferManifolds.Add(Manifold);
						}
						//TransferedConstraint->SetManifoldPoints(TransferManifolds);
						TransferedConstraint->ResetActiveManifoldContacts();
						for (FManifoldPoint& Manifold : CollisionConstraint.GetManifoldPoints())
						{
							TransferedConstraint->AddOneshotManifoldContact(Manifold.ContactPoint);
						}

						FReal CollisionConstraintStiffness = JointConstraint->GetSettings().ContactTransferScale;
						if (TransferCollisionsDebugTestAgainstMaxClamp)
						{
							CollisionConstraintStiffness = TransferCollisionsStiffnessClamp;
						}

						if (CollisionParticleA.Get() && CollisionParticleA->ObjectState() != EObjectStateType::Dynamic)
						{
							if (ensureMsgf(TransferCollisionsKinematicScale > 0, TEXT("Zero or Negative TransferCollisionsKinematicScale")))
							{
								CollisionConstraintStiffness *= TransferCollisionsKinematicScale;
							}
						}
						TransferedConstraint->SetStiffness(FMath::Clamp(CollisionConstraintStiffness, (FReal)0.f, TransferCollisionsStiffnessClamp));

						FParticlePairMidPhase* MidPhase = CollisionAllocator.GetParticlePairMidPhase(CollisionParticleA->Handle(), CollisionParticleB->Handle());
						MidPhase->InjectCollision(*TransferedConstraint);

						return ECollisionVisitorResult::Continue;
					});
			}
		}

		CollisionAllocator.ProcessInjectedConstraints();
	}
#endif
}

void FPBDRigidsEvolutionGBF::DestroyParticleCollisionsInAllocator(FGeometryParticleHandle* Particle)
{
	if (Particle != nullptr)
	{
		CollisionConstraints.GetConstraintAllocator().RemoveParticle(Particle);
		Particle->ParticleCollisions().Reset();
	}
}

void FPBDRigidsEvolutionGBF::DestroyTransientConstraints(FGeometryParticleHandle* Particle)
{
	if (Particle != nullptr)
	{
		// Remove all the particle's collisions from the graph
		GetIslandManager().RemoveParticleContainerConstraints(Particle, CollisionConstraints.GetContainerId());

		// Mark all collision constraints for destruction
		DestroyParticleCollisionsInAllocator(Particle);
	}
}

void FPBDRigidsEvolutionGBF::DestroyTransientConstraints()
{
	for (FTransientGeometryParticleHandle& Particle : Particles.GetAllParticlesView())
	{
		DestroyTransientConstraints(Particle.Handle());
	}
}

void FPBDRigidsEvolutionGBF::SetParticleTransform(FGeometryParticleHandle* InParticle, const FVec3& InPos, const FRotation3& InRot, const bool bIsTeleport, const bool bWakeUp)
{
	const FVec3 PrevX = InParticle->GetX();
	const FRotation3 PrevR = InParticle->GetR();

	FGenericParticleHandle(InParticle)->SetTransform(InPos, InRot);

	// We only want to set the IsParticleTransformSetExplicitly flag if the particle will be woken up and this is a teleport.
	// Since only then we want to estimate an overlap velocity based on penetration.
	// (see @FPBDCollisionConstraints::AddConstraintsToGraph)
	if (bWakeUp && bIsTeleport)
	{
		GetIslandManager().FlagParticleTransformSetExplicitly(InParticle);
	}
	GetIslandManager().FlagParticleUpdated(InParticle);

	OnParticleMoved(InParticle, PrevX, PrevR, bIsTeleport, bWakeUp);

	// This is a bit overkill, if this becomes an issue we will need to look into adding support
	// to record partial updates of particles, only the transform in this case
	CVD_TRACE_PARTICLE(InParticle);

#if CHAOS_EVOLUTION_COLLISION_TESTMODE
	{
		// Update the test mode cache so we can move particles in PIE to test collisions
		TestModeUpdateSavedParticle(InParticle);
	}
#endif
}

void FPBDRigidsEvolutionGBF::SetParticleTransformSwept(FGeometryParticleHandle* InParticle, const FVec3& InPos, const FRotation3& InRot, const bool bIsTeleport)
{
	// If the rotation has changed, we need to adjust the initial sweep position to account for the fact that the
	// particle will have moved in a straight line through its center of mass. This is important when we have shapes
	// that are not centered on their center of mass (either it's multi-shape body, or the user modified the CoM).
	const FVec3 CoMOffset = FConstGenericParticleHandle(InParticle)->CenterOfMass();
	const FVec3 EndCoM = InPos + InRot * CoMOffset;
	const FVec3 StartCoM = InParticle->GetX() + InParticle->GetR() * CoMOffset;
	FVec3 SweepDir = EndCoM - StartCoM;
	const FReal SweepLength = SweepDir.SafeNormalize();
	const FVec3 SweepStart = InPos - SweepDir * SweepLength;

	FReal TOI = FReal(1.0);
	if (SweepLength > UE_SMALL_NUMBER)
	{
		Private::FSimSweepParticleHit Hit;
		if (Private::SimSweepParticleFirstHit(GetSpatialAcceleration(), &GetBroadPhase().GetIgnoreCollisionManager(), InParticle, SweepStart, InRot, SweepDir, SweepLength, Hit))
		{
			TOI = Hit.HitTOI;
		}
	}

	if (TOI > 0)
	{
		const FVec3 NewPos = FMath::Lerp(SweepStart, InPos, TOI);
		SetParticleTransform(InParticle, NewPos, InRot, bIsTeleport, true);
	}
}

bool FPBDRigidsEvolutionGBF::SetParticleKinematicTarget(FGeometryParticleHandle* ParticleHandle, const FKinematicTarget& NewKinematicTarget)
{
	bool bIsAddedToParticles = false;
	FGenericParticleHandle Particle = ParticleHandle;

	// NOTE: If called on a dynamic body we just move the body
	// @todo(chaos): maybe we should ensure that this is not called for dynamics
	if (Particle->IsKinematic())
	{
		FKinematicGeometryParticleHandle* Kinematic = Particle->CastToKinematicParticle();

		// optimization : we keep track of moving kinematic targets ( list gets clear every frame )
		if (NewKinematicTarget.GetMode() != EKinematicTargetMode::None)
		{
			// move particle from "non-moving" kinematics to "moving" kinematics
			Particles.MarkMovingKinematic(Kinematic);
			bIsAddedToParticles = true;
		}
		Kinematic->SetKinematicTarget(NewKinematicTarget);
	}
	else if (Particle->IsDynamic())
	{
		if (NewKinematicTarget.GetMode() == EKinematicTargetMode::Position)
		{
			SetParticleTransform(ParticleHandle, NewKinematicTarget.GetPosition(), NewKinematicTarget.GetRotation(), false);
		}
	}
	return bIsAddedToParticles;
}

void FPBDRigidsEvolutionGBF::OnParticleMoved(FGeometryParticleHandle* InParticle, const FVec3& PrevX, const FRotation3& PrevR, const bool bIsTeleport, const bool bWakeUp)
{
	// When a particle is moved, we need to 
	// - tell the collisions because they cache friction state and would attempt to undo small translations within the friction cone
	// - wake the island(s) that the particle is in
	// NOTE: we have a tolerance on the transform change because SetParticleTransform may be called with the "same" transform that
	// is different by very small amounts around 1e-7 in both position and rotation when switching from dynamic to kinematic.
	const FReal CollisionPositionTolerance = FReal(1.e-4);
	const FReal CollisionRotationTolerance = FReal(1.e-6);
	if (!FVec3::IsNearlyEqual(PrevX, InParticle->GetX(), CollisionPositionTolerance) || !FRotation3::IsNearlyEqual(PrevR, InParticle->GetR(), CollisionRotationTolerance))
	{
		if (bWakeUp)
		{
			GetIslandManager().WakeParticleIslands(InParticle);
		}

		InParticle->ParticleCollisions().VisitCollisions([this, InParticle](FPBDCollisionConstraint& Collision)
			{
				Collision.UpdateParticleTransform(InParticle);
				return ECollisionVisitorResult::Continue;
			});
	}

	// If this is a teleport, we assume other state has been set separately (V, W, etc) if required, which 
	// would include resetting any sleep-related state. If this is not a teleport, we want to prevent 
	// sleeping if the user keeps moving the body without changing the velocity.
	// If the particle transform was set explicitly (contains check for partial island sleep), 
	// we also want to manually set VSmooth to make use of momentum propagation in FPBDIslandManager::UpdateExplicitSleep.
	if (!bIsTeleport || GetIslandManager().WasParticleTransformSetExplicitly(InParticle))
	{
		if (FPBDRigidParticleHandle* Rigid = InParticle->CastToRigidParticle())
		{
			// No need to do this for kinematics
			if (Rigid->IsDynamic())
			{
				// @todo(chaos): better sleep system! We should probably have a sleep accumulator per particle
				const FReal InvDt = FReal(30.0);
				const FVec3 DV = (PrevX - Rigid->GetX()) * InvDt;
				const FReal SmoothRate = FMath::Clamp(CVars::SmoothedPositionLerpRate, 0.0f, 1.0f);
				Rigid->VSmooth() = FMath::Lerp(Rigid->VSmooth(), Rigid->GetV() + DV, SmoothRate);
			}

			if (Rigid->IsSleeping())
			{
				SetParticleObjectState(Rigid, EObjectStateType::Dynamic);
			}
		}
	}
}

void FPBDRigidsEvolutionGBF::SetParticleDynamics(FPBDRigidParticleHandle* Rigid, const FParticleDynamics& Dynamics)
{
	Rigid->SetDynamics(Dynamics);
	ResetVSmoothFromForces(*Rigid);
	GetIslandManager().FlagParticleUpdated(Rigid->Handle());
}

void FPBDRigidsEvolutionGBF::SetParticleMassProps(FPBDRigidParticleHandle* Rigid, const FParticleMassProps& MassProperties)
{
	Rigid->SetMassProps(MassProperties);
	GetIslandManager().FlagParticleUpdated(Rigid->Handle());
}

void FPBDRigidsEvolutionGBF::ApplyParticleTransformCorrectionDelta(FGeometryParticleHandle* InParticle, const FVec3& InPosDelta, const FVec3& InRotDelta, const bool bApplyToConnectedBodies, const bool bInRecalculateFrictionOnConnectedBodies, const TArray<FParticleID>& ExcludeConnections)
{
	ApplyParticleTransformCorrection(
		InParticle, 
		InParticle->GetX() + InPosDelta, 
		FRotation3::IntegrateRotationWithAngularVelocity(InParticle->GetR(), InRotDelta, FReal(1.0)),
		bApplyToConnectedBodies, bInRecalculateFrictionOnConnectedBodies, ExcludeConnections);
}

void FPBDRigidsEvolutionGBF::ApplyParticleTransformCorrection(FGeometryParticleHandle* InParticle, const FVec3& InPos, const FRotation3& InRot, const bool bApplyToConnectedBodies, const bool bInRecalculateFrictionOnConnectedBodies, const TArray<FParticleID>& ExcludeConnections)
{
	const FRigidTransform3 OldParticleTransform = InParticle->GetTransformXR();
	const FRigidTransform3 NewParticleTransform = FRigidTransform3(InPos, InRot);

	// Move the root particle
	ApplyParticleTransformCorrectionImpl(InParticle, NewParticleTransform);

	if (bApplyToConnectedBodies)
	{
		// Find all the connected particles and move them to retain their relative transform
		// NOTE: ConnectedParticles will not include InParticle
		TArray<FGeometryParticleHandle*> ConnectedParticles = GetConnectedParticles(InParticle, ExcludeConnections);
		for (FGeometryParticleHandle* ConnectedParticle : ConnectedParticles)
		{
			if (FGenericParticleHandle(ConnectedParticle)->IsDynamic())
			{
				const FRigidTransform3 RelativeTransform = ConnectedParticle->GetTransformXR().GetRelativeTransformNoScale(OldParticleTransform);
				const FRigidTransform3 NewOtherParticleTransform = RelativeTransform * NewParticleTransform;
				ApplyParticleTransformCorrectionImpl(ConnectedParticle, NewOtherParticleTransform, bInRecalculateFrictionOnConnectedBodies);
			}
		}
	}
}

void FPBDRigidsEvolutionGBF::ApplyParticleTransformCorrectionImpl(FGeometryParticleHandle* InParticle, const FRigidTransform3& InTransform, const bool bInRecalculateFriction)
{
	FGenericParticleHandle Particle = InParticle;

	Particle->SetX(InTransform.GetTranslation());
	Particle->SetP(InTransform.GetTranslation());
	Particle->SetR(InTransform.GetRotation());
	Particle->SetQ(InTransform.GetRotation());

	if (bInRecalculateFriction)
	{
		// We must fix the collision anchors so that friction doesn't undo our move or rotation
		InParticle->ParticleCollisions().VisitCollisions(
			[this, InParticle](FPBDCollisionConstraint& Collision)
			{
				Collision.UpdateParticleTransform(InParticle);
				return ECollisionVisitorResult::Continue;
			});
	}
}

void FPBDRigidsEvolutionGBF::ApplySleepOnConnectedParticles(FGeometryParticleHandle* InParticle)
{
	TArray<FGeometryParticleHandle*> ConnectedParticles = GetConnectedParticles(InParticle);
	for (FGeometryParticleHandle* ConnectedParticle : ConnectedParticles)
	{
		if (FGenericParticleHandle(ConnectedParticle)->ObjectState() == EObjectStateType::Dynamic)
		{
			if (FPBDRigidParticleHandle* Rigid = ConnectedParticle->CastToRigidParticle())
			{
				SetParticleObjectState(Rigid, EObjectStateType::Sleeping);
			}
		}
	}
}

TArray<FGeometryParticleHandle*> FPBDRigidsEvolutionGBF::GetConnectedParticles(FGeometryParticleHandle* InParticle, const TArray<FParticleID>& ExcludeConnections)
{
	if (InParticle->ParticleConstraints().IsEmpty())
	{
		return {};
	}

	// Use a hashmap to prevent O(N^2) loop below. It will also contains the output list of connected particles.
	struct FHashMapTraits
	{
		static uint32 GetIDHash(const FGeometryParticleHandle* Particle) { return MurmurFinalize32(uint32(Particle->UniqueIdx().Idx)); }
		static const FGeometryParticleHandle* GetElementID(const FGeometryParticleHandle* Element) { return Element; }
	};
	Private::THashMappedArray<FGeometryParticleHandle*, FGeometryParticleHandle*, FHashMapTraits> ConnectedParticles(256);

	FGeometryParticleHandle* NextParticle = InParticle;
	int32 NextParticleIndex = 0;
	while (true)
	{
		// Loop over the joints on the next particle and add the other particle into the queue (if not already in the queue, or already processed)
		for (FConstraintHandle* Constraint : NextParticle->ParticleConstraints())
		{
			if (FPBDJointConstraintHandle* Joint = Constraint->As<FPBDJointConstraintHandle>())
			{
				if (Joint->IsConstraintEnabled() && !Joint->IsConstraintBroken())
				{
					const TVec3<EJointMotionType>& JointLinearMotion = Joint->GetSettings().LinearMotionTypes;
					if ((JointLinearMotion[0] == EJointMotionType::Locked) && (JointLinearMotion[1] == EJointMotionType::Locked) && (JointLinearMotion[2] == EJointMotionType::Locked))
					{
						// Get the other particle on the joint
						FParticlePair JointParticles = Joint->GetConstrainedParticles();
						FGeometryParticleHandle* OtherParticle = (JointParticles[0] != NextParticle) ? JointParticles[0] : JointParticles[1];

						// NOTE: We do not generate connections through kinematic particles
						if ((OtherParticle != nullptr) && (OtherParticle != InParticle) && FGenericParticleHandle(OtherParticle)->IsDynamic())
						{
							if ((ConnectedParticles.Find(OtherParticle) == nullptr) && !ExcludeConnections.Contains(OtherParticle->ParticleID()))
							{
								ConnectedParticles.Add(OtherParticle, OtherParticle);
							}
						}
					}
				}
			}
		}

		// We are done if we did not add any particles in the most recent loop above
		check(NextParticleIndex <= ConnectedParticles.Num());
		if (NextParticleIndex >= ConnectedParticles.Num())
		{
			break;
		}

		// Move to the next particle in the queue
		NextParticle = ConnectedParticles.At(NextParticleIndex);
		++NextParticleIndex;
	}
	check(NextParticleIndex == ConnectedParticles.Num());

	// NOTE: This moves the internal array to the output - no duplication
	return ConnectedParticles.ExtractElements();
}

void FPBDRigidsEvolutionGBF::SetParticleVelocities(FGeometryParticleHandle* InParticle, const FVec3& InV, const FVec3f& InW)
{
	if (FKinematicGeometryParticleHandle* Kinematic = InParticle->CastToKinematicParticle())
	{
		Kinematic->SetV(InV);
		Kinematic->SetW(InW);
	}

	if (FPBDRigidParticleHandle* Rigid = InParticle->CastToRigidParticle())
	{
		// Reset the velocity-based sleepiness tracking properties
		if (Rigid->IsDynamic())
		{
			Rigid->SetVSmooth(InV);
			Rigid->SetWSmooth(InW);

			// Wake the particle if the velocity is non-zero
			// NOTE: We do this even when not sleeping because we want to reset the sleep accumulators
			if (!InV.IsNearlyZero() || !InW.IsNearlyZero())
			{
				WakeParticle(Rigid);
			}

			GetIslandManager().FlagParticleUpdated(Rigid->Handle());

			// @todo(chaos): do we want to reset static friction an any existing contacts as well?
		}
	}
}

void FPBDRigidsEvolutionGBF::ParticleMaterialChanged(FGeometryParticleHandle* Particle)
{
	Particle->ParticleCollisions().VisitCollisions([this](FPBDCollisionConstraint& Collision)
	{
		// Reset the material. This is a fast operation - the material properties will get collected later if the collision is activated
		Collision.ClearMaterialProperties();
		return ECollisionVisitorResult::Continue;

	}, ECollisionVisitorFlags::VisitAllCurrentAndExpired);

	// The graph caches some sleep thresholds etc 
	GetIslandManager().UpdateParticleMaterial(Particle);
}

const FChaosPhysicsMaterial* FPBDRigidsEvolutionGBF::GetFirstClusteredPhysicsMaterial(const FGeometryParticleHandle* Particle) const
{
	if (const FChaosPhysicsMaterial* PhysicsMaterial = GetFirstPhysicsMaterial(Particle))
	{
		return PhysicsMaterial;
	}
	else if (const FPBDRigidClusteredParticleHandle* Cluster = Particle->CastToClustered())
	{
		if (const FClusterUnion* ClusterUnion = Clustering.GetClusterUnionManager().FindClusterUnionFromParticle(Cluster))
		{
			if (ClusterUnion->InternalCluster == Cluster)
			{
				for (const FPBDRigidParticleHandle* ChildParticle : ClusterUnion->ChildParticles)
				{
					// I'm not sure how a cluster might end up containing itself, but
					// in order to avoid this at all costs we just add a check here to make
					// sure the particle isn't it's own child before trying to get its
					// material
					if (ChildParticle && ensureMsgf(ChildParticle != Particle, TEXT("Clustered particle found which has itself as a child")))
					{
						if (const FChaosPhysicsMaterial* ChildPhysicalMaterial = GetFirstClusteredPhysicsMaterial(ChildParticle))
						{
							return ChildPhysicalMaterial;
						}
					}
				}
			}
		}
	}

	return nullptr;
}

void FPBDRigidsEvolutionGBF::UpdateInertiaConditioning()
{
	// The maximum contribution to error correction from rotation
	const FRealSingle MaxRotationRatio = ChaosSolverInertiaConditioningRotationRatio;

	// The error distance that the constraint correction must be stable for
	// @todo(chaos): should probably be tied to constraint teleport threshold?
	const FRealSingle MaxDistance = ChaosSolverInertiaConditioningDistance;

	// A limit on the relative sizes of the inertia components (inverse)
	const FRealSingle MaxInvInertiaComponentRatio = ChaosSolverMaxInvInertiaComponentRatio;

	// Tolerances (currently controlled by cvar)
	const FInertiaConditioningTolerances& Tolerances = ChaosInertiaConditioningTolerances;

	// Did the settings change?
	const bool bSettingsChanged = (InertiaConditioningEpoch != ChaosSolverInertiaConditioningEpoch);
	InertiaConditioningEpoch = ChaosSolverInertiaConditioningEpoch;

	// @todo(chaos): we should only visit particles that are dirty
	for (auto& Rigid : Particles.GetNonDisabledDynamicView())
	{
		// Update the inertia conditioning if some property of the body, or the system settings have changed
		if (Rigid.InertiaConditioningDirty() || (bSettingsChanged && Rigid.InertiaConditioningEnabled()))
		{
			const bool bWantInertiaConditioning = Rigid.IsDynamic() && Rigid.InertiaConditioningEnabled() && bChaosSolverInertiaConditioningEnabled;
			if (bWantInertiaConditioning)
			{
				const FVec3f InvInertiaScale = CalculateParticleInertiaConditioning(Rigid.Handle(), MaxDistance, MaxRotationRatio, MaxInvInertiaComponentRatio, Tolerances);
				Rigid.SetInvIConditioning(InvInertiaScale);
			}
			else
			{
				Rigid.SetInvIConditioning(FVec3f(1));
			}
			Rigid.ClearInertiaConditioningDirty();
		}
	}
}

#if CHAOS_EVOLUTION_COLLISION_TESTMODE
void FPBDRigidsEvolutionGBF::TestModeResetCollisions()
{
	if (!CVars::bChaos_Solver_TestMode_Enabled)
	{
		return;
	}
	
	for (FPBDCollisionConstraintHandle* Collision : CollisionConstraints.GetConstraintHandles())
	{
		if (Collision != nullptr)
		{
			Collision->GetContact().ResetManifold();
			Collision->GetContact().ResetModifications();
			Collision->GetContact().GetGJKWarmStartData().Reset();
		}
	}
}
#endif
}

