// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/PBDRigidsSOAs.h"

namespace Chaos
{
	void FPBDRigidsSOAs::UpdateViews()
	{
		//build various views. Group SOA types together for better branch prediction
		{
			auto TmpArray = bResimulating
				? TArray<TSOAView<FGeometryParticles>>
			{
				// re-sim only works with a reduced set of particle types, i.e. no DynamicClusteredMapArray (DynamicGeometryCollectionArray??) BH
				{&ResimStaticParticles.GetArray()},
				{&ResimKinematicParticles.GetArray()},
				{&ResimDynamicParticles.GetArray()},
				{&ResimDynamicKinematicParticles.GetArray()}
			}
			: TArray<TSOAView<FGeometryParticles>>
			{
				StaticParticles.Get(),
				KinematicParticles.Get(),
				DynamicParticles.Get(),
				DynamicKinematicParticles.Get(),
				{&StaticClusteredMapArray.GetArray()},
				{&KinematicClusteredMapArray.GetArray()},
				{&DynamicClusteredMapArray.GetArray()},
				{&StaticGeometryCollectionArray.GetArray()},
				{&KinematicGeometryCollectionArray.GetArray()},
				{&SleepingGeometryCollectionArray.GetArray()},
				{&DynamicGeometryCollectionArray.GetArray()}
			};
			NonDisabledView = MakeParticleView(MoveTemp(TmpArray));
		}
		{
			auto TmpArray = bResimulating
				? TArray<TSOAView<FPBDRigidParticles>>
			{
				{&ResimDynamicParticles.GetArray() }
			}
			: TArray<TSOAView<FPBDRigidParticles>>
			{
				DynamicParticles.Get(),
				{&DynamicClusteredMapArray.GetArray() },
				{&SleepingGeometryCollectionArray.GetArray() },
				{&DynamicGeometryCollectionArray.GetArray() }
			};
			NonDisabledDynamicView = MakeParticleView(MoveTemp(TmpArray));
		}
		{
			TArray<TSOAView<FPBDRigidParticles>> TmpArray =
			{
				{&ActiveParticlesMapArray.GetArray()},
				{&MovingKinematicsMapArray.GetArray()},
				//{&KinematicClusteredMapArray.GetArray()},	// @todo(chaos): we should include moving clustered kinematics in this view
			};
			ActiveDynamicMovingKinematicParticlesView = MakeParticleView(MoveTemp(TmpArray));
		}
		{
			auto TmpArray = bResimulating
				? TArray<TSOAView<FPBDRigidParticles>>
			{
				{&ResimActiveParticlesMapArray.GetArray()}
			}
			: TArray<TSOAView<FPBDRigidParticles>>
			{
				{&ActiveParticlesMapArray.GetArray()},
				{&StaticGeometryCollectionArray.GetArray()},
				{&KinematicGeometryCollectionArray.GetArray()},
			};
			ActiveParticlesView = MakeParticleView(MoveTemp(TmpArray));
		}
		{
			auto TmpArray = bResimulating
				? TArray<TSOAView<FPBDRigidParticles>>
			{
				{&ResimActiveParticlesMapArray.GetArray()},
				{&TransientDirtyMapArray.GetArray()}
			}
			: TArray<TSOAView<FPBDRigidParticles>>
			{
				{&ActiveParticlesMapArray.GetArray()},
				{&MovingKinematicsMapArray.GetArray()},
				{&SleepingGeometryCollectionArray.GetArray()},
				{&TransientDirtyMapArray.GetArray()}
			};
			DirtyParticlesView = MakeParticleView(MoveTemp(TmpArray));
		}
		{
			TArray<TSOAView<FGeometryParticles>> TmpArray =
			{
				StaticParticles.Get(),
				StaticDisabledParticles.Get(),
				KinematicParticles.Get(),
				KinematicDisabledParticles.Get(),
				DynamicParticles.Get(),
				DynamicDisabledParticles.Get(),
				DynamicKinematicParticles.Get(),
				ClusteredParticles.Get(),
				GeometryCollectionParticles.Get()
			};
			AllParticlesView = MakeParticleView(MoveTemp(TmpArray));
		}
		{
			auto TmpArray = bResimulating
				? TArray<TSOAView<FKinematicGeometryParticles>>
			{
				{&ResimKinematicParticles.GetArray()},
				{ &ResimDynamicKinematicParticles.GetArray() }
			}
			: TArray<TSOAView<FKinematicGeometryParticles>>
			{
				KinematicParticles.Get(),
				DynamicKinematicParticles.Get(),
				{&KinematicGeometryCollectionArray.GetArray()},
				{&KinematicClusteredMapArray.GetArray()}
			};
			ActiveKinematicParticlesView = MakeParticleView(MoveTemp(TmpArray));
		}
		{
			// todo(chaos) handle resim ?
			TArray<TSOAView<FPBDRigidParticles>> TmpArray =
			{
				{&MovingKinematicsMapArray.GetArray()},
			};
			ActiveMovingKinematicParticlesView = MakeParticleView(MoveTemp(TmpArray));
		}
		{
			auto TmpArray = bResimulating
				? TArray<TSOAView<FGeometryParticles>>
			{
				{&ResimStaticParticles.GetArray()}
			}
			: TArray<TSOAView<FGeometryParticles>>
			{
				StaticParticles.Get(),
				{&StaticClusteredMapArray.GetArray()}
			};
			ActiveStaticParticlesView = MakeParticleView(MoveTemp(TmpArray));
		}
		{
			auto TmpArray = bResimulating
				? TArray<TSOAView<TPBDGeometryCollectionParticles<FReal, 3>>> {}	//no geometry collection during resim
			: TArray<TSOAView<TPBDGeometryCollectionParticles<FReal, 3>>>
			{
				{&StaticGeometryCollectionArray.GetArray()},
				{&KinematicGeometryCollectionArray.GetArray()},
				{&DynamicGeometryCollectionArray.GetArray()}
			};
			ActiveGeometryCollectionParticlesView = MakeParticleView(MoveTemp(TmpArray));
		}

		{
			auto TmpArray = bResimulating
				? TArray<TSOAView<FPBDRigidClusteredParticles>> {}	//no clusters during resim
			: TArray<TSOAView<FPBDRigidClusteredParticles>>
			{
				{&StaticClusteredMapArray.GetArray()},
				{&KinematicClusteredMapArray.GetArray()},
				{&DynamicClusteredMapArray.GetArray()}
			};
			NonDisabledClusteredView = MakeParticleView(MoveTemp(TmpArray));
		}
	}

	void FPBDRigidsSOAs::SetContainerListMasks()
	{
		StaticParticles->SetContainerListMask(EGeometryParticleListMask::StaticParticles);
		StaticDisabledParticles->SetContainerListMask(EGeometryParticleListMask::StaticDisabledParticles);
		KinematicParticles->SetContainerListMask(EGeometryParticleListMask::KinematicParticles);
		KinematicDisabledParticles->SetContainerListMask(EGeometryParticleListMask::KinematicDisabledParticles);
		DynamicDisabledParticles->SetContainerListMask(EGeometryParticleListMask::DynamicDisabledParticles);
		DynamicParticles->SetContainerListMask(EGeometryParticleListMask::DynamicParticles);
		DynamicKinematicParticles->SetContainerListMask(EGeometryParticleListMask::DynamicKinematicParticles);
		ClusteredParticles->SetContainerListMask(EGeometryParticleListMask::ClusteredParticles);
		GeometryCollectionParticles->SetContainerListMask(EGeometryParticleListMask::GeometryCollectionParticles);

		StaticGeometryCollectionArray.SetContainerListMask(EGeometryParticleListMask::StaticGeometryCollectionArray);
		KinematicGeometryCollectionArray.SetContainerListMask(EGeometryParticleListMask::KinematicGeometryCollectionArray);
		SleepingGeometryCollectionArray.SetContainerListMask(EGeometryParticleListMask::SleepingGeometryCollectionArray);
		DynamicGeometryCollectionArray.SetContainerListMask(EGeometryParticleListMask::DynamicGeometryCollectionArray);
		ActiveParticlesMapArray.SetContainerListMask(EGeometryParticleListMask::ActiveParticlesMapArray);
		TransientDirtyMapArray.SetContainerListMask(EGeometryParticleListMask::TransientDirtyMapArray);
		MovingKinematicsMapArray.SetContainerListMask(EGeometryParticleListMask::MovingKinematicsMapArray);
		ResimActiveParticlesMapArray.SetContainerListMask(EGeometryParticleListMask::ResimActiveParticlesMapArray);
		ResimDynamicParticles.SetContainerListMask(EGeometryParticleListMask::ResimDynamicParticles);
		ResimDynamicKinematicParticles.SetContainerListMask(EGeometryParticleListMask::ResimDynamicKinematicParticles);
		ResimStaticParticles.SetContainerListMask(EGeometryParticleListMask::ResimStaticParticles);
		ResimKinematicParticles.SetContainerListMask(EGeometryParticleListMask::ResimKinematicParticles);
		StaticClusteredMapArray.SetContainerListMask(EGeometryParticleListMask::StaticClusteredMapArray);
		KinematicClusteredMapArray.SetContainerListMask(EGeometryParticleListMask::KinematicClusteredMapArray);
		DynamicClusteredMapArray.SetContainerListMask(EGeometryParticleListMask::DynamicClusteredMapArray);
	}

	void FPBDRigidsSOAs::CheckListMasks()
	{
		CheckSOAMasks(StaticParticles.Get());
		CheckSOAMasks(StaticDisabledParticles.Get());
		CheckSOAMasks(KinematicParticles.Get());
		CheckSOAMasks(KinematicDisabledParticles.Get());
		CheckSOAMasks(DynamicDisabledParticles.Get());
		CheckSOAMasks(DynamicParticles.Get());
		CheckSOAMasks(DynamicKinematicParticles.Get());
		CheckSOAMasks(ClusteredParticles.Get());
		CheckSOAMasks(GeometryCollectionParticles.Get());

		CheckListMasks(StaticGeometryCollectionArray);
		CheckListMasks(KinematicGeometryCollectionArray);
		CheckListMasks(SleepingGeometryCollectionArray);
		CheckListMasks(DynamicGeometryCollectionArray);
		CheckListMasks(ActiveParticlesMapArray);
		CheckListMasks(TransientDirtyMapArray);
		CheckListMasks(MovingKinematicsMapArray);
		CheckListMasks(ResimActiveParticlesMapArray);
		CheckListMasks(ResimDynamicParticles);
		CheckListMasks(ResimDynamicKinematicParticles);
		CheckListMasks(ResimStaticParticles);
		CheckListMasks(ResimKinematicParticles);
		CheckListMasks(StaticClusteredMapArray);
		CheckListMasks(KinematicClusteredMapArray);
		CheckListMasks(DynamicClusteredMapArray);
	}

	void FPBDRigidsSOAs::CheckViewMasks()
	{
		for (auto& Particle : GetAllParticlesView())
		{
			CheckParticleViewMaskImpl(Particle.Handle());
		}
	}

	// Check that the particle's list mask contains only one of the bits from a view's list mask
	bool ZeroOrOneBitSet(const EGeometryParticleListMask ParticleListMask, const EGeometryParticleListMask ViewListMask)
	{
		const EGeometryParticleListMask ParticleViewListMask = ParticleListMask & ViewListMask;
		return FMath::IsPowerOfTwo(uint32(ParticleViewListMask));
	}

	void FPBDRigidsSOAs::CheckParticleViewMaskImpl(const FGeometryParticleHandle* Particle) const
	{
		const EGeometryParticleListMask ParticleListMask = Particle->ListMask();

		// All particles must be in a list or SOA
		ensureAlwaysMsgf(!(ParticleListMask == EGeometryParticleListMask::None), TEXT("Particle %s is not in any lists"), *GetParticleDebugName(*Particle));

		// Cannot be in more than one of these lists at a time
		// @todo(chaos): we could automate ViewListMask creation if the Views knew what ParticleMapArrays and 
		// ParticleArrays they contained but currently they just holds a pointer to the underlying TArray.

		// NonDisabledDynamicView is used by collision detection. Duplicates in this view can lead to race conditions in the midphase
		const EGeometryParticleListMask NonDisabledDynamicViewListMask = 
			EGeometryParticleListMask::DynamicParticles 
			| EGeometryParticleListMask::DynamicClusteredMapArray 
			| EGeometryParticleListMask::SleepingGeometryCollectionArray 
			| EGeometryParticleListMask::DynamicGeometryCollectionArray;
		ensureAlwaysMsgf(ZeroOrOneBitSet(ParticleListMask, NonDisabledDynamicViewListMask), TEXT("Particle %s duplicated in NonDisabledDynamicView (ListMask 0x%08x)"), *GetParticleDebugName(*Particle), EnumToUnderlyingType(ParticleListMask));
	
		const EGeometryParticleListMask ActiveDynamicMovingKinematicParticlesViewListMask = 
			EGeometryParticleListMask::ActiveParticlesMapArray
			| EGeometryParticleListMask::MovingKinematicsMapArray;
		ensureAlwaysMsgf(ZeroOrOneBitSet(ParticleListMask, ActiveDynamicMovingKinematicParticlesViewListMask), TEXT("Particle %s duplicated in ActiveDynamicMovingKinematicParticlesView (ListMask 0x%08x)"), *GetParticleDebugName(*Particle), EnumToUnderlyingType(ParticleListMask));
	}
}