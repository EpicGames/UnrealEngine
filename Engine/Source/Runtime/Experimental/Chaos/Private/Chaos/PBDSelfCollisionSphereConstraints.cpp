// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/PBDSelfCollisionSphereConstraints.h"

#if !COMPILE_WITHOUT_UNREAL_SUPPORT
#include "Chaos/Framework/Parallel.h"
#include "Chaos/HierarchicalSpatialHash.h"
#include "HAL/IConsoleManager.h"

namespace Chaos::Softs
{
	namespace Private
	{
		static int32 Chaos_SelfCollisionSphere_ParallelCount = 256;
		static FAutoConsoleVariableRef CVarChaosSelfCollisionSphereParallelismEnabled(TEXT("p.Chaos.SelfCollisionSphere.ParallelCount"), Chaos_SelfCollisionSphere_ParallelCount, TEXT("Min number of spheres to enable parallelism for SelfCollisionSpheres constraint"));

		struct FSphereSpatialEntry
		{
			const TConstArrayView<FSolverVec3>* Points;
			int32 Index;

			FSolverVec3 X() const
			{
				return (*Points)[Index];
			}

			template<typename TPayloadType>
			int32 GetPayload(int32) const
			{
				return Index;
			}
		};
	}

	FPBDSelfCollisionSphereConstraintsBase::FPBDSelfCollisionSphereConstraintsBase(
		const int32 InOffset,
		const int32 InNumParticles,
		const TSet<int32>* InVertexSetNoOffset,
		const TArray<FSolverVec3>* InReferencePositions,
		const FSolverReal InRadius,
		const FSolverReal InStiffness)
		: Radius(InRadius)
		, Stiffness(InStiffness)
		, VertexSetNoOffset(InVertexSetNoOffset)
		, Offset(InOffset)
		, NumParticles(InNumParticles)
		, ReferencePositions(InReferencePositions)
	{
	}

	template<typename SolverParticlesOrRange>
	void FPBDSelfCollisionSphereConstraintsBase::Init(const SolverParticlesOrRange& Particles)
	{
		Constraints.Reset();

		if (NumParticles == 0 || (VertexSetNoOffset && VertexSetNoOffset->IsEmpty()) || Stiffness <= 0.f)
		{
			return;
		}

		// Build Spatial
		TArray<Private::FSphereSpatialEntry> Entries;
		const FSolverReal Diameter = 2.f * Radius;
		TConstArrayView<FSolverVec3> Points = Particles.XArray();
		if (VertexSetNoOffset)
		{
			Entries.Reset(VertexSetNoOffset->Num());
			for (const int32 VertexNoOffset : *VertexSetNoOffset)
			{
				Entries.Add({ &Points, VertexNoOffset + Offset });
			}
		}
		else
		{
			Entries.Reset(NumParticles);
			for (int32 Index = 0; Index < NumParticles; ++Index)
			{
				Entries.Add({ &Points, Index + Offset });
			}
		}
		TSpatialHashGridPoints<int32, FSolverReal> SpatialHash(Diameter);
		SpatialHash.InitializePoints(Entries, Entries.Num() < Private::Chaos_SelfCollisionSphere_ParallelCount);
		
		const FSolverReal DiamSq = FMath::Square(Diameter);
		constexpr int32 CellRadius = 1; // We set the cell size of the spatial hash such that we only need to look 1 cell away to find proximities.
		constexpr int32 MaxNumExpectedConnectionsPerParticle = 3;
		const int32 MaxNumExpectedConnections = MaxNumExpectedConnectionsPerParticle * Entries.Num();

		const TConstArrayView<FSolverVec3> ReferencePositionsView = ReferencePositions ? Particles.GetConstArrayView(*ReferencePositions) : TConstArrayView<FSolverVec3>();
		Constraints = SpatialHash.FindAllSelfProximities(CellRadius, MaxNumExpectedConnections,
			[this, &Particles, DiamSq, &ReferencePositionsView](const int32 i1, const int32 i2)
		{
			const FSolverReal CombinedMass = Particles.InvM(i1) + Particles.InvM(i2);
			if (CombinedMass < (FSolverReal)1e-7)
			{
				return false;
			}
			if (ReferencePositions)
			{
				if (FSolverVec3::DistSquared(ReferencePositionsView[i1], ReferencePositionsView[i2]) < DiamSq)
				{
					return false;
				}
			}
			return true;
		}, Entries.Num() < Private::Chaos_SelfCollisionSphere_ParallelCount
		);
	}
	template CHAOS_API void FPBDSelfCollisionSphereConstraintsBase::Init(const FSolverParticles& Particles);
	template CHAOS_API void FPBDSelfCollisionSphereConstraintsBase::Init(const FSolverParticlesRange& Particles);

	template<typename SolverParticlesOrRange>
	void FPBDSelfCollisionSphereConstraintsBase::Apply(SolverParticlesOrRange& Particles, const FSolverReal Dt) const
	{
		const FSolverReal Diameter = 2.f * Radius;
		const FSolverReal DiamSq = FMath::Square(Diameter);
		for (const TVec2<int32>& Constraint : Constraints)
		{
			const int32 i1 = Constraint[0];
			const int32 i2 = Constraint[1];
			const FSolverReal CombinedMass = Particles.InvM(i1) + Particles.InvM(i2);
			checkSlow(CombinedMass >= (FSolverReal)1e-7);

			FSolverVec3& P1 = Particles.P(i1);
			FSolverVec3& P2 = Particles.P(i2);
			const FSolverVec3 Difference = P1 - P2;
			const FSolverReal DistSq = Difference.SizeSquared();
			if (DistSq >= DiamSq)
			{
				continue;
			}
			const FSolverReal Dist = FMath::Sqrt(DistSq);
			const FSolverVec3 Delta = Stiffness * (Dist - Diameter) * Difference / (CombinedMass * Dist);
			if (Particles.InvM(i1) > (FSolverReal)0.)
			{
				P1 -= Particles.InvM(i1) * Delta;
			}
			if (Particles.InvM(i2) > (FSolverReal)0.)
			{
				P2 += Particles.InvM(i2) * Delta;
			}
		}
	}
	template CHAOS_API void FPBDSelfCollisionSphereConstraintsBase::Apply(FSolverParticles& Particles, const FSolverReal Dt) const;
	template CHAOS_API void FPBDSelfCollisionSphereConstraintsBase::Apply(FSolverParticlesRange& Particles, const FSolverReal Dt) const;

	void FPBDSelfCollisionSphereConstraints::SetProperties(
		const FCollectionPropertyConstFacade& PropertyCollection,
		const TMap<FString, const TSet<int32>*>& VertexSets)
	{
		if (IsSelfCollisionSphereRadiusMutable(PropertyCollection))
		{
			Radius = (FSolverReal)FMath::Max(GetSelfCollisionSphereRadius(PropertyCollection), 0.f);
		}
		if (IsSelfCollisionSphereStiffnessMutable(PropertyCollection))
		{
			Stiffness = (FSolverReal)FMath::Clamp(GetSelfCollisionSphereStiffness(PropertyCollection), 0.f, 1.f);
		}
	}
}  // End namespace Chaos::Softs

#endif
