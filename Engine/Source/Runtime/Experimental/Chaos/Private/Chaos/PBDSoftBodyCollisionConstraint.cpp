// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/PBDSoftBodyCollisionConstraint.h"
#include "Chaos/Levelset.h"
#include "Chaos/MLLevelset.h"
#include "Chaos/SoftsEvolutionLinearSystem.h"
#include "Chaos/TaperedCapsule.h"
#include "Chaos/WeightedLatticeImplicitObject.h"
#include "ChaosStats.h"
#include "HAL/IConsoleManager.h"
#include "Chaos/Framework/Parallel.h"
#include "Misc/ScopeLock.h"
#if INTEL_ISPC
#include "PBDSoftBodyCollisionConstraint_Simple.ispc.generated.h"
#include "PBDSoftBodyCollisionConstraint_Complex.ispc.generated.h"
#include "PerParticlePBDCollisionConstraintISPCDataVerification.h"


#if !UE_BUILD_SHIPPING
bool bChaos_SoftBodyCollision_ISPC_Enabled = CHAOS_PER_PARTICLE_COLLISION_ISPC_ENABLED_DEFAULT;
static FAutoConsoleVariableRef CVarChaosSoftBodyCollisionISPCEnabled(TEXT("p.Chaos.SoftBodyCollision.ISPC"), bChaos_SoftBodyCollision_ISPC_Enabled, TEXT("Whether to use ISPC optimizations in per particle collisions"));
#endif
#endif

static int32 Chaos_SoftBodyCollision_ISPC_ParallelBatchSize = 128;
#if !UE_BUILD_SHIPPING
FAutoConsoleVariableRef CVarChaosSoftBodyCollisionISPCParallelBatchSize(TEXT("p.Chaos.SoftBodyCollision.ISPC.ParallelBatchSize"), Chaos_SoftBodyCollision_ISPC_ParallelBatchSize, TEXT("Parallel batch size for ISPC"));
#endif

DECLARE_CYCLE_STAT(TEXT("Chaos PBD Collision Constraint Apply"), STAT_ChaosPBDCollisionConstraintApply, STATGROUP_Chaos);

namespace Chaos::Softs {

	namespace Private
	{
		static bool IsComplexBatchCollider(Chaos::EImplicitObjectType CollisionType)
		{
			return CollisionType == TWeightedLatticeImplicitObject<FLevelSet>::StaticType() || CollisionType == Chaos::ImplicitObjectType::MLLevelSet;
		}

		static bool IsSimpleCollider(Chaos::EImplicitObjectType CollisionType)
		{
			if (IsComplexBatchCollider(CollisionType))
			{
				return false;
			}

			if(CollisionType == Chaos::ImplicitObjectType::SkinnedTriangleMesh)
			{
				// Note: SkinnedTriangleMesh collisions are handled by FPBDSkinnedTriangleMeshCollisions
				return false;
			}

			return true;
		}

		static void ApplyFrictionAndReturnVelocity(FSolverVec3& P, const FSolverVec3& X, const FSolverVec3& NormalWorld, const FSolverReal MaxFrictionCorrection, const FSolverReal Dt, const FSolverVec3& CollisionX, const FSolverVec3& CollisionV, const FSolverVec3& CollisionW, FSolverVec3& ColliderVelocityAtPoint)
		{
			const FSolverVec3 VectorToPoint = P - CollisionX;
			ColliderVelocityAtPoint = CollisionV + FSolverVec3::CrossProduct(CollisionW, VectorToPoint);

			const FSolverVec3 RelativeDisplacement = (P - X) - ColliderVelocityAtPoint * Dt; // This corresponds to the tangential velocity multiplied by dt (friction will drive this to zero if it is high enough)
			const FSolverVec3 RelativeDisplacementTangent = RelativeDisplacement - FSolverVec3::DotProduct(RelativeDisplacement, NormalWorld) * NormalWorld; // Project displacement into the tangential plane
			const FSolverReal RelativeDisplacementTangentLength = RelativeDisplacementTangent.Size();
			if (RelativeDisplacementTangentLength >= UE_SMALL_NUMBER)
			{
				const FSolverReal PositionCorrection = FMath::Min<FSolverReal>(MaxFrictionCorrection, RelativeDisplacementTangentLength);
				const FSolverReal CorrectionRatio = PositionCorrection / RelativeDisplacementTangentLength;
				P -= CorrectionRatio * RelativeDisplacementTangent;
			}
		}

		static void ApplyFriction(FSolverVec3& P, const FSolverVec3& X, const FSolverVec3& NormalWorld, const FSolverReal MaxFrictionCorrection, const FSolverReal Dt, const FSolverVec3& CollisionX, const FSolverVec3& CollisionV, const FSolverVec3& CollisionW)
		{
			FSolverVec3 ColliderVelocityAtPoint;
			ApplyFrictionAndReturnVelocity(P, X, NormalWorld, MaxFrictionCorrection, Dt, CollisionX, CollisionV, CollisionW, ColliderVelocityAtPoint);
		}

		// Defined in PerParticlePBDCollisionConstraint.cpp
		extern void ReflectOneSidedCollision(const FSolverVec3& P, const FSolverVec3& OneSidedPlaneNormal, const FSolverVec3& SplitOrigin, FSolverReal& Penetration, FSolverVec3& ImplicitNormal);

		template<bool bWithFriction>
		static inline bool ApplySimpleCCD(
			const FSolverCollisionParticlesRange& CollisionParticlesRange,
			const int32 CollisionIndex,
			const FSolverRigidTransform3& LastCollisionTransform,
			const FSolverVec3& X,
			const FSolverReal Thickness,
			[[maybe_unused]] const FSolverReal FrictionCoefficient,
			[[maybe_unused]] const FSolverReal Dt,
			FSolverVec3& P,
			FSolverVec3& NormalWorld,
			FSolverVec3& ContactWorld,
			FSolverReal& Penetration)
		{
			const FSolverRigidTransform3 Frame(CollisionParticlesRange.GetX(CollisionIndex), CollisionParticlesRange.R(CollisionIndex));
			const FVec3 StartPoint(LastCollisionTransform.InverseTransformPositionNoScale(X));  // Geometry operates in FReal
			const FVec3 EndPoint(Frame.InverseTransformPositionNoScale(P));

			const FImplicitObjectPtr& Geometry = CollisionParticlesRange.GetGeometry(CollisionIndex);
			const Pair<FVec3, bool> PointPair = Geometry->FindClosestIntersection(StartPoint, EndPoint, (FReal)Thickness);

			if (PointPair.Second)
			{
				const FSolverVec3 Normal = FSolverVec3(Geometry->Normal(PointPair.First));
				NormalWorld = Frame.TransformVectorNoScale(Normal);
				ContactWorld = Frame.TransformPositionNoScale(UE::Math::TVector<FSolverReal>(PointPair.First));

				const FSolverVec3 Direction = ContactWorld - P;
				Penetration = FMath::Max((FSolverReal)0., FSolverVec3::DotProduct(NormalWorld, Direction)) + (FSolverReal)UE_THRESH_POINT_ON_PLANE;

				P += Penetration * NormalWorld;

				if constexpr (bWithFriction)
				{
					// Friction
					const FSolverVec3 CollisionV = CollisionParticlesRange.V(CollisionIndex);
					const FSolverVec3 CollisionX = CollisionParticlesRange.X(CollisionIndex);
					const FSolverVec3 CollisionW = CollisionParticlesRange.W(CollisionIndex);
					Private::ApplyFriction(P, X, NormalWorld, Penetration * FrictionCoefficient, Dt, CollisionX, CollisionV, CollisionW);
				}
				return true;
			}
			return false;
		}
	}

// Helper function to call ApplySimpleCCD and return data to ISPC
extern "C" void ApplySimpleCCDNoFriction(
	const uint8* CollisionParticlesRange,
	const int32 CollisionIndex,
	const FSolverRigidTransform3& LastCollisionTransform,
	const FSolverReal* XSoa,
	const FSolverReal* Thickness,
	FSolverReal* PSoa,
	const int32 ProgramCount,
	const int32 Mask)
{
	for (int32 Index = 0; Index < ProgramCount; ++Index)
	{
		if (Mask & (1 << Index))
		{
			const FSolverVec3 X(
				XSoa[Index],
				XSoa[Index + ProgramCount],
				XSoa[Index + 2 * ProgramCount]);
			FSolverVec3 P(
				PSoa[Index],
				PSoa[Index + ProgramCount],
				PSoa[Index + 2 * ProgramCount]);
			constexpr bool bWithFriction = false;
			FSolverVec3 Normal;
			FSolverVec3 Contact;
			FSolverReal Penetration;
			Private::ApplySimpleCCD<bWithFriction>(
				*(const FSolverCollisionParticlesRange*)CollisionParticlesRange,
				CollisionIndex,
				LastCollisionTransform,
				X,
				Thickness[Index],
				(FSolverReal)0., // No friction, no FrictionCoefficient needed
				(FSolverReal)0., // No friction, no Dt needed
				P,
				Normal,
				Contact,
				Penetration);
			PSoa[Index] = P.X;
			PSoa[Index + ProgramCount] = P.Y;
			PSoa[Index + 2 * ProgramCount] = P.Z;
		}
	}
}

extern "C" void ApplySimpleCCDWithFriction(
	const uint8* CollisionParticlesRange,
	const int32 CollisionIndex,
	const FSolverRigidTransform3& LastCollisionTransform,
	const FSolverReal* XSoa,
	const FSolverReal* Thickness,
	const FSolverReal FrictionCoefficient,
	const FSolverReal Dt,
	FSolverReal* PSoa,
	const int32 ProgramCount,
	const int32 Mask)
{
	for (int32 Index = 0; Index < ProgramCount; ++Index)
	{
		if (Mask & (1 << Index))
		{
			const FSolverVec3 X(
				XSoa[Index],
				XSoa[Index + ProgramCount],
				XSoa[Index + 2 * ProgramCount]);
			FSolverVec3 P(
				PSoa[Index],
				PSoa[Index + ProgramCount],
				PSoa[Index + 2 * ProgramCount]);
			constexpr bool bWithFriction = true;
			FSolverVec3 Normal;
			FSolverVec3 Contact;
			FSolverReal Penetration;
			Private::ApplySimpleCCD<bWithFriction>(
				*(const FSolverCollisionParticlesRange*)CollisionParticlesRange,
				CollisionIndex,
				LastCollisionTransform,
				X,
				Thickness[Index],
				FrictionCoefficient,
				Dt,
				P,
				Normal,
				Contact,
				Penetration);
			PSoa[Index] = P.X;
			PSoa[Index + ProgramCount] = P.Y;
			PSoa[Index + 2 * ProgramCount] = P.Z;
		}
	}
}

// Helper function to call PhiWithNormal and return data to ISPC
extern "C" void GetPhiWithNormalCollisionParticleRange(const uint8 * CollisionParticlesRange, const FSolverReal * InV, FSolverReal * Normal, FSolverReal * Phi, const int32 i, const int32 ProgramCount, const int32 Mask)
{
	const FSolverCollisionParticlesRange& C = *(const FSolverCollisionParticlesRange*)CollisionParticlesRange;	
	for (int32 Index = 0; Index < ProgramCount; ++Index)
	{
		if (Mask & (1 << Index))
		{
			FSolverVec3 V;

			// aos_to_soa3
			V.X = InV[Index];
			V.Y = InV[Index + ProgramCount];
			V.Z = InV[Index + 2 * ProgramCount];

			FVec3 ImplicitNormal;
			Phi[Index] = (FSolverReal)C.GetGeometry(i)->PhiWithNormal(FVec3(V), ImplicitNormal);
			FSolverVec3 Norm(ImplicitNormal);

			// aos_to_soa3
			Normal[Index] = Norm.X;
			Normal[Index + ProgramCount] = Norm.Y;
			Normal[Index + 2 * ProgramCount] = Norm.Z;
		}
	}
}

void FPBDSoftBodyCollisionConstraintBase::ApplyWithPlanarConstraints(FSolverParticlesRange& Particles, const FSolverReal Dt, const TArray<FSolverCollisionParticlesRange>& CollisionParticles, bool bGeneratePlanarConstraints)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPBDSoftBodyCollisionConstraint_ApplyWithPlanarConstraints);

	if (CollisionParticles.IsEmpty())
	{
		return;
	}

	const bool bLockAndWriteContacts = bWriteDebugContacts && CollisionParticleCollided && Contacts && Normals && Phis;
	const bool bWithFriction = FrictionCoefficient > (FSolverReal)UE_KINDA_SMALL_NUMBER;
	if (bUseCCD)
	{
		if (bWithFriction)
		{
			if (bLockAndWriteContacts)
			{
				ApplyInternalCCD<true, true>(Particles, Dt, CollisionParticles);
			}
			else
			{
				ApplyInternalCCD<false, true>(Particles, Dt, CollisionParticles);
			}
		}
		else
		{
			if (bLockAndWriteContacts)
			{
				ApplyInternalCCD<true, false>(Particles, Dt, CollisionParticles);
			}
			else
			{
				ApplyInternalCCD<false, false>(Particles, Dt, CollisionParticles);
			}
		}
	}
	else
	{
		if (bGeneratePlanarConstraints)
		{
			InitPlanarConstraints(Particles, bWithFriction);
#if INTEL_ISPC
			if (bChaos_SoftBodyCollision_ISPC_Enabled && bRealTypeCompatibleWithISPC)
			{
				ApplySimpleInternalISPC(Particles, Dt, CollisionParticles, bUsePlanarConstraintForSimpleColliders);
				ApplyComplexInternalISPC(Particles, Dt, CollisionParticles, bUsePlanarConstraintForComplexColliders);
			}
			else
#endif
			{
				if (bLockAndWriteContacts)
				{
					if (bWithFriction)
					{
						if (bUsePlanarConstraintForSimpleColliders)
						{
							ApplySimpleInternal<true, true, true>(Particles, Dt, CollisionParticles);
						}
						else
						{
							ApplySimpleInternal<true, true, false>(Particles, Dt, CollisionParticles);
						}
						if (bUsePlanarConstraintForComplexColliders)
						{
							ApplyComplexInternal<true, true, true>(Particles, Dt, CollisionParticles);
						}
						else
						{
							ApplyComplexInternal<true, true, false>(Particles, Dt, CollisionParticles);
						}
					}
					else
					{
						if (bUsePlanarConstraintForSimpleColliders)
						{
							ApplySimpleInternal<true, false, true>(Particles, Dt, CollisionParticles);
						}
						else
						{
							ApplySimpleInternal<true, false, false>(Particles, Dt, CollisionParticles);
						}
						if (bUsePlanarConstraintForComplexColliders)
						{
							ApplyComplexInternal<true, false, true>(Particles, Dt, CollisionParticles);
						}
						else
						{
							ApplyComplexInternal<true, false, false>(Particles, Dt, CollisionParticles);
						}
					}
				}
				else
				{
					if (bWithFriction)
					{
						if (bUsePlanarConstraintForSimpleColliders)
						{
							ApplySimpleInternal<false, true, true>(Particles, Dt, CollisionParticles);
						}
						else
						{
							ApplySimpleInternal<false, true, false>(Particles, Dt, CollisionParticles);
						}
						if (bUsePlanarConstraintForComplexColliders)
						{
							ApplyComplexInternal<false, true, true>(Particles, Dt, CollisionParticles);
						}
						else
						{
							ApplyComplexInternal<false, true, false>(Particles, Dt, CollisionParticles);
						}
					}
					else
					{
						if (bUsePlanarConstraintForSimpleColliders)
						{
							ApplySimpleInternal<false, false, true>(Particles, Dt, CollisionParticles);
						}
						else
						{
							ApplySimpleInternal<false, false, false>(Particles, Dt, CollisionParticles);
						}
						if (bUsePlanarConstraintForComplexColliders)
						{
							ApplyComplexInternal<false, false, true>(Particles, Dt, CollisionParticles);
						}
						else
						{
							ApplyComplexInternal<false, false, false>(Particles, Dt, CollisionParticles);
						}
					}
				}
			}
			FinalizePlanarConstraints(Particles);
		}
		else
		{
#if INTEL_ISPC
			if (bChaos_SoftBodyCollision_ISPC_Enabled && bRealTypeCompatibleWithISPC)
			{
				if (!bUsePlanarConstraintForSimpleColliders)
				{
					ApplySimpleInternalISPC(Particles, Dt, CollisionParticles, false);
				}
				if (!bUsePlanarConstraintForComplexColliders)
				{
					ApplyComplexInternalISPC(Particles, Dt, CollisionParticles, false);
				}
			}
			else
#endif
			{
				if (bLockAndWriteContacts)
				{
					if (bWithFriction)
					{
						if (!bUsePlanarConstraintForSimpleColliders)
						{
							ApplySimpleInternal<true, true, false>(Particles, Dt, CollisionParticles);
						}
						if (!bUsePlanarConstraintForComplexColliders)
						{
							ApplyComplexInternal<true, true, false>(Particles, Dt, CollisionParticles);
						}
					}
					else
					{
						if (!bUsePlanarConstraintForSimpleColliders)
						{
							ApplySimpleInternal<true, false, false>(Particles, Dt, CollisionParticles);
						}
						if (!bUsePlanarConstraintForComplexColliders)
						{
							ApplyComplexInternal<true, false, false>(Particles, Dt, CollisionParticles);
						}
					}
				}
				else
				{
					if (bWithFriction)
					{
						if (!bUsePlanarConstraintForSimpleColliders)
						{
							ApplySimpleInternal<false, true, false>(Particles, Dt, CollisionParticles);
						}
						if (!bUsePlanarConstraintForComplexColliders)
						{
							ApplyComplexInternal<false, true, false>(Particles, Dt, CollisionParticles);
						}
					}
					else
					{
						if (!bUsePlanarConstraintForSimpleColliders)
						{
							ApplySimpleInternal<false, false, false>(Particles, Dt, CollisionParticles);
						}
						if (!bUsePlanarConstraintForComplexColliders)
						{
							ApplyComplexInternal<false, false, false>(Particles, Dt, CollisionParticles);
						}
					}
				}
			}
			ApplyPlanarConstraints(Particles, Dt);
		}
	}
}


void FPBDSoftBodyCollisionConstraintBase::Apply(FSolverParticlesRange& Particles, const FSolverReal Dt, const TArray<FSolverCollisionParticlesRange>& CollisionParticles)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPBDSoftBodyCollisionConstraint);

	if (CollisionParticles.IsEmpty())
	{
		return;
	}

	const bool bLockAndWriteContacts = bWriteDebugContacts && CollisionParticleCollided && Contacts && Normals && Phis;
	const bool bWithFriction = FrictionCoefficient > (FSolverReal)UE_KINDA_SMALL_NUMBER;
	if (bUseCCD)
	{
		if (bWithFriction)
		{
			if (bLockAndWriteContacts)
			{
				ApplyInternalCCD<true, true>(Particles, Dt, CollisionParticles);
			}
			else
			{
				ApplyInternalCCD<false, true>(Particles, Dt, CollisionParticles);
			}
		}
		else
		{
			if (bLockAndWriteContacts)
			{
				ApplyInternalCCD<true, false>(Particles, Dt, CollisionParticles);
			}
			else
			{
				ApplyInternalCCD<false, false>(Particles, Dt, CollisionParticles);
			}
		}
	}
	else
	{
#if INTEL_ISPC
		if (bChaos_SoftBodyCollision_ISPC_Enabled && bRealTypeCompatibleWithISPC)
		{
			ApplySimpleInternalISPC(Particles, Dt, CollisionParticles, false);
			ApplyComplexInternalISPC(Particles, Dt, CollisionParticles, false);
		}
		else
#endif
		{
			if (bLockAndWriteContacts)
			{
				if (bWithFriction)
				{
					ApplySimpleInternal<true, true, false>(Particles, Dt, CollisionParticles);
					ApplyComplexInternal<true, true, false>(Particles, Dt, CollisionParticles);
				}
				else
				{
					ApplySimpleInternal<true, false, false>(Particles, Dt, CollisionParticles);
					ApplyComplexInternal<true, false, false>(Particles, Dt, CollisionParticles);
				}
			}
			else
			{
				if (bWithFriction)
				{
					ApplySimpleInternal<false, true, false>(Particles, Dt, CollisionParticles);
					ApplyComplexInternal<false, true, false>(Particles, Dt, CollisionParticles);
				}
				else
				{
					ApplySimpleInternal<false, false, false>(Particles, Dt, CollisionParticles);
					ApplyComplexInternal<false, false, false>(Particles, Dt, CollisionParticles);
				}
			}
		}
	}
}

void FPBDSoftBodyCollisionConstraintBase::InitPlanarConstraints(const FSolverParticlesRange& Particles, bool bWithFriction)
{
	// Initialize Planar Constraint.
	PlanarConstraint.Reset();
	HasPlanarData.Init(false, Particles.GetRangeSize());
	PlanarDataPositions.SetNumUninitialized(Particles.GetRangeSize());
	PlanarDataNormals.SetNumUninitialized(Particles.GetRangeSize());
	if (bWithFriction)
	{
		PlanarDataVelocities.SetNumUninitialized(Particles.GetRangeSize());
	}
}

template<bool bLockAndWriteContacts, bool bWithFriction, bool bGeneratePlanarConstraints>
void FPBDSoftBodyCollisionConstraintBase::ApplySimpleInternal(FSolverParticlesRange& Particles, const FSolverReal Dt, const TArray<FSolverCollisionParticlesRange>& CollisionParticles)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPBDSoftBodyCollisionConstraint_ApplySimpleInternal);

	if (!bEnableSimpleColliders)
	{
		return;
	}

	FPAndInvM* const PAndInvM = Particles.GetPAndInvM().GetData();
	const FSolverVec3* const X = Particles.XArray().GetData();

	PhysicsParallelFor(Particles.GetRangeSize(), [this, &X, &PAndInvM, &CollisionParticles, Dt](int32 Index)
		{
			if (PAndInvM[Index].InvM == (FSolverReal)0.)
			{
				return;
			}
			const FSolverReal InnerCollisionThicknessValue = InnerCollisionThickness.HasWeightMap() ? InnerCollisionThickness[Index] : (FSolverReal)InnerCollisionThickness;

			for (const FSolverCollisionParticlesRange& CollisionParticlesRange : CollisionParticles)
			{
				const TConstArrayView<FSolverRigidTransform3> CollisionTransforms = CollisionParticlesRange.GetConstArrayView(LastCollisionTransforms);

				for (int32 CollisionIndex = 0; CollisionIndex < CollisionParticlesRange.GetRangeSize(); ++CollisionIndex)
				{
					const FImplicitObjectPtr& Geometry = CollisionParticlesRange.GetGeometry(CollisionIndex);
					const Chaos::EImplicitObjectType CollisionType = Geometry->GetType();
					if (Private::IsComplexBatchCollider(CollisionType))
					{
						continue;
					}

					const FSolverRigidTransform3 Frame(CollisionParticlesRange.GetX(CollisionIndex), CollisionParticlesRange.R(CollisionIndex));
					const FVec3 RigidSpacePosition(Frame.InverseTransformPosition(PAndInvM[Index].P));  // PhiWithNormal requires FReal based arguments
					FVec3 ImplicitNormalReal;
					const FSolverReal Phi = (FSolverReal)Geometry->PhiWithNormal(RigidSpacePosition, ImplicitNormalReal);
					FSolverVec3 ImplicitNormal(ImplicitNormalReal);
					const FSolverReal Thickness = SoftBodyCollisionThicknessWeighted.GetValue(Index) + CollisionThickness;
					FSolverReal Penetration = Thickness - Phi;
					if (Penetration > (FSolverReal)0.)
					{
						// Split capsules always push out in the OneSidedPlaneNormal direction.
						if (CollisionType == Chaos::ImplicitObjectType::TaperedCapsule)
						{
							const FTaperedCapsule& Capsule = Geometry->GetObjectChecked<FTaperedCapsule>();
							if (Capsule.IsOneSided())
							{
								Private::ReflectOneSidedCollision(FSolverVec3(RigidSpacePosition), Capsule.GetOneSidedPlaneNormalf(), Capsule.GetOriginf(), Penetration, ImplicitNormal);
							}
						}

						// Fallback to CCD when the penetration is deep enough to cause large projection errors
						if (InnerCollisionThicknessValue > (FSolverReal)0. && Penetration > InnerCollisionThicknessValue)
						{
							FSolverVec3 NormalWorld;
							FSolverVec3 ContactWorld;
							if (Private::ApplySimpleCCD<bWithFriction>(
								CollisionParticlesRange,
								CollisionIndex,
								CollisionTransforms[CollisionIndex],
								X[Index],
								Thickness,
								FrictionCoefficient,
								Dt,
								PAndInvM[Index].P,
								NormalWorld,
								ContactWorld,
								Penetration))
							{
								if constexpr (bLockAndWriteContacts)
								{
									FScopeLock Lock(&DebugMutex);
									CollisionParticlesRange.GetArrayView(*CollisionParticleCollided)[CollisionIndex] = true;
									Contacts->Emplace(ContactWorld);
									Normals->Emplace(NormalWorld);
									Phis->Emplace(Penetration);
								}
							}
							continue;
						}

						const FSolverVec3 NormalWorld = Frame.TransformVector(ImplicitNormal);
						if constexpr (bLockAndWriteContacts)
						{
							FScopeLock Lock(&DebugMutex);
							CollisionParticlesRange.GetArrayView(*CollisionParticleCollided)[CollisionIndex] = true;
							Contacts->Emplace(PAndInvM[Index].P);
							Normals->Emplace(NormalWorld);
							Phis->Emplace(Phi);
						}

						PAndInvM[Index].P += Penetration * NormalWorld;

						if constexpr (bGeneratePlanarConstraints)
						{
							// Last collider per point wins.
							HasPlanarData[Index] = true;
							PlanarDataPositions[Index] = PAndInvM[Index].P;
							PlanarDataNormals[Index] = NormalWorld;
						}

						if constexpr (bWithFriction)
						{
							FSolverVec3 ColliderVelocityAtPoint;
							Private::ApplyFrictionAndReturnVelocity(
								PAndInvM[Index].P,
								X[Index],
								NormalWorld,
								Penetration * FrictionCoefficient,
								Dt,
								CollisionParticlesRange.X(CollisionIndex),
								CollisionParticlesRange.V(CollisionIndex),
								CollisionParticlesRange.W(CollisionIndex),
								ColliderVelocityAtPoint);

							if constexpr (bGeneratePlanarConstraints)
							{
								PlanarDataVelocities[Index] = ColliderVelocityAtPoint;
							}
						}
					}
				}
			}
		}
	);
}

template<bool bLockAndWriteContacts, bool bWithFriction, bool bGeneratePlanarConstraints>
void FPBDSoftBodyCollisionConstraintBase::ApplyComplexInternal(FSolverParticlesRange& Particles, const FSolverReal Dt, const TArray<FSolverCollisionParticlesRange>& CollisionParticles)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPBDSoftBodyCollisionConstraint_ApplyComplexInternal);
	if (!bEnableComplexColliders)
	{
		return;
	}

	FPAndInvM* const PAndInvM = Particles.GetPAndInvM().GetData();
	const FSolverVec3* const X = Particles.XArray().GetData();
	for (const FSolverCollisionParticlesRange& CollisionParticlesRange : CollisionParticles)
	{
		for (int32 CollisionIndex = 0; CollisionIndex < CollisionParticlesRange.GetRangeSize(); ++CollisionIndex)
		{
			const FImplicitObjectPtr& Geometry = CollisionParticlesRange.GetGeometry(CollisionIndex);
			const Chaos::EImplicitObjectType CollisionType = Geometry->GetType();
			if (!Private::IsComplexBatchCollider(CollisionType))
			{
				continue;
			}

			const FSolverRotation3& CollisionR = CollisionParticlesRange.R(CollisionIndex);
			const FSolverVec3& CollisionV = CollisionParticlesRange.V(CollisionIndex);
			const FSolverVec3& CollisionX = CollisionParticlesRange.X(CollisionIndex);
			const FSolverVec3& CollisionW = CollisionParticlesRange.W(CollisionIndex);

			bool* const bCollided = bLockAndWriteContacts ? &CollisionParticlesRange.GetArrayView(*CollisionParticleCollided)[CollisionIndex] : nullptr;
			const FSolverRigidTransform3 Frame(CollisionX, CollisionR);
			if constexpr (bWithFriction)
			{
				if (const FPBDComplexColliderBoneData* const ColliderBoneData = ComplexBoneData.Find(FParticleRangeIndex(CollisionParticlesRange.GetRangeId(), CollisionIndex)))
				{
					if (const TWeightedLatticeImplicitObject<FLevelSet>* const LevelSet = Geometry->GetObject<TWeightedLatticeImplicitObject<FLevelSet>>())
					{
						LevelSet->BatchPhiWithNormalAndGreatestInfluenceBone(Particles.GetPAndInvM(), Frame, FMath::Max(SoftBodyCollisionThicknessWeighted.GetLow(), SoftBodyCollisionThicknessWeighted.GetHigh()) + CollisionThickness, BatchPhis, BatchNormals, BatchVelocityBones);
					}
					else
					{
						checkNoEntry();
					}

					// Apply
					PhysicsParallelFor(Particles.GetRangeSize(), [this, &X, &PAndInvM, &Frame, &CollisionV, &CollisionX, &CollisionW, ColliderBoneData, Dt, &bCollided](int32 Index)
						{
							if (PAndInvM[Index].InvM == (FSolverReal)0.)
							{
								return;
							}

							const FSolverReal Penetration = SoftBodyCollisionThicknessWeighted.GetValue(Index) + CollisionThickness - BatchPhis[Index];
							if (Penetration > (FSolverReal)0.)
							{
								const FSolverVec3 NormalWorld = Frame.TransformVector(BatchNormals[Index]);
								if constexpr (bLockAndWriteContacts)
								{
									FScopeLock Lock(&DebugMutex);
									*bCollided = true;
									Contacts->Emplace(PAndInvM[Index].P);
									Normals->Emplace(NormalWorld);
									Phis->Emplace(BatchPhis[Index]);
								}

								PAndInvM[Index].P += Penetration * NormalWorld;

								if constexpr (bGeneratePlanarConstraints)
								{
									// Last collider per point wins.
									HasPlanarData[Index] = true;
									PlanarDataPositions[Index] = PAndInvM[Index].P;
									PlanarDataNormals[Index] = NormalWorld;
								}

								FSolverVec3 ColliderVelocityAtPoint;
								const int32 StrongestBone = BatchVelocityBones[Index];
								if (ColliderBoneData->MappedBoneIndices.IsValidIndex(StrongestBone))
								{
									const int32 MappedIndex = ColliderBoneData->MappedBoneIndices[StrongestBone];
									Private::ApplyFrictionAndReturnVelocity(PAndInvM[Index].P, X[Index], NormalWorld, Penetration * FrictionCoefficient, Dt, ColliderBoneData->X[MappedIndex], ColliderBoneData->V[MappedIndex], ColliderBoneData->W[MappedIndex], ColliderVelocityAtPoint);
								}
								else
								{
									Private::ApplyFrictionAndReturnVelocity(PAndInvM[Index].P, X[Index], NormalWorld, Penetration * FrictionCoefficient, Dt, CollisionX, CollisionV, CollisionW, ColliderVelocityAtPoint);
								}

								if constexpr (bGeneratePlanarConstraints)
								{
									PlanarDataVelocities[Index] = ColliderVelocityAtPoint;
								}
							}
						}
					);
				}
				else
				{
					if (const FMLLevelSet* const MLLevelSet = Geometry->GetObject<FMLLevelSet>())
					{
						BatchPhis.SetNumUninitialized(Particles.GetRangeSize());
						BatchNormals.SetNumUninitialized(Particles.GetRangeSize());

						// Batch Query
						constexpr int32 MLLevelSetThreadNum = 0;
						MLLevelSet->BatchPhiWithNormal(Particles.GetPAndInvM(), Frame, BatchPhis, BatchNormals, FMath::Max(SoftBodyCollisionThicknessWeighted.GetLow(), SoftBodyCollisionThicknessWeighted.GetHigh()) + CollisionThickness, MLLevelSetThreadNum, 0, Particles.GetRangeSize());
					}
					else if (const TWeightedLatticeImplicitObject<FLevelSet>* const LevelSet = Geometry->GetObject<TWeightedLatticeImplicitObject<FLevelSet>>())
					{
						LevelSet->BatchPhiWithNormal(Particles.GetPAndInvM(), Frame, BatchPhis, BatchNormals);
					}
					else
					{
						checkNoEntry();
					}

					// Apply
					PhysicsParallelFor(Particles.GetRangeSize(), [this, &X, &PAndInvM, &CollisionV, &CollisionX, &CollisionW, &Frame, Dt, &bCollided](int32 Index)
						{
							if (PAndInvM[Index].InvM == (FSolverReal)0.)
							{
								return;
							}

							const FSolverReal Penetration = SoftBodyCollisionThicknessWeighted.GetValue(Index) + CollisionThickness - BatchPhis[Index];
							if (Penetration > (FSolverReal)0.)
							{
								const FSolverVec3 NormalWorld = Frame.TransformVector(BatchNormals[Index]);
								if constexpr (bLockAndWriteContacts)
								{
									FScopeLock Lock(&DebugMutex);
									*bCollided = true;
									Contacts->Emplace(PAndInvM[Index].P);
									Normals->Emplace(NormalWorld);
									Phis->Emplace(BatchPhis[Index]);
								}

								PAndInvM[Index].P += Penetration * NormalWorld;

								if constexpr (bGeneratePlanarConstraints)
								{
									// Last collider per point wins.
									HasPlanarData[Index] = true;
									PlanarDataPositions[Index] = PAndInvM[Index].P;
									PlanarDataNormals[Index] = NormalWorld;
								}

								FSolverVec3 ColliderVelocityAtPoint;
								Private::ApplyFrictionAndReturnVelocity(PAndInvM[Index].P, X[Index], NormalWorld, Penetration * FrictionCoefficient, Dt, CollisionX, CollisionV, CollisionW, ColliderVelocityAtPoint);

								if (bGeneratePlanarConstraints)
								{
									PlanarDataVelocities[Index] = ColliderVelocityAtPoint;
								}
							}
						}
					);
				}
			}
			else
			{
				if (const FMLLevelSet* const MLLevelSet = Geometry->GetObject<FMLLevelSet>())
				{
					BatchPhis.SetNumUninitialized(Particles.GetRangeSize());
					BatchNormals.SetNumUninitialized(Particles.GetRangeSize());

					// Batch Query
					constexpr int32 MLLevelSetThreadNum = 0;
					MLLevelSet->BatchPhiWithNormal(Particles.GetPAndInvM(), Frame, BatchPhis, BatchNormals, FMath::Max(SoftBodyCollisionThicknessWeighted.GetLow(), SoftBodyCollisionThicknessWeighted.GetHigh()) + CollisionThickness, MLLevelSetThreadNum, 0, Particles.GetRangeSize());
				}
				else if (const TWeightedLatticeImplicitObject<FLevelSet>* const LevelSet = Geometry->GetObject<TWeightedLatticeImplicitObject<FLevelSet>>())
				{
					LevelSet->BatchPhiWithNormal(Particles.GetPAndInvM(), Frame, BatchPhis, BatchNormals);
				}
				else
				{
					checkNoEntry();
				}

				// Apply
				PhysicsParallelFor(Particles.GetRangeSize(), [this, &X, &PAndInvM, &Frame, &bCollided](int32 Index)
					{
						if (PAndInvM[Index].InvM == (FSolverReal)0.)
						{
							return;
						}

						const FSolverReal Penetration = SoftBodyCollisionThicknessWeighted.GetValue(Index) + CollisionThickness - BatchPhis[Index];
						if (Penetration > (FSolverReal)0.)
						{
							const FSolverVec3 NormalWorld = Frame.TransformVector(BatchNormals[Index]);
							if constexpr (bLockAndWriteContacts)
							{
								FScopeLock Lock(&DebugMutex);
								*bCollided = true;
								Contacts->Emplace(PAndInvM[Index].P);
								Normals->Emplace(NormalWorld);
								Phis->Emplace(BatchPhis[Index]);
							}

							PAndInvM[Index].P += Penetration * NormalWorld;

							if constexpr (bGeneratePlanarConstraints)
							{
								// Last collider per point wins.
								HasPlanarData[Index] = true;
								PlanarDataPositions[Index] = PAndInvM[Index].P;
								PlanarDataNormals[Index] = NormalWorld;
							}
						}
					}
				);
			}
		}
	}
}

void FPBDSoftBodyCollisionConstraintBase::FinalizePlanarConstraints(const FSolverParticlesRange& Particles)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPBDSoftBodyCollisionConstraint_FinalizePlanarConstraints);
	// Count how many planar constraints we have.
	int32 NumPlanarConstraints = 0;
	for (const bool bHasPlanarData : HasPlanarData)
	{
		if (bHasPlanarData)
		{
			++NumPlanarConstraints;
		}
	}

	PlanarConstraint.GetUniqueConstraintIndices().SetNumUninitialized(NumPlanarConstraints);
	PlanarConstraint.GetTargetPositions().SetNumUninitialized(NumPlanarConstraints);
	PlanarConstraint.GetTargetNormals().SetNumUninitialized(NumPlanarConstraints);
	PlanarConstraint.GetTargetVelocities().SetNumUninitialized(NumPlanarConstraints);

	if (NumPlanarConstraints > 0)
	{
		int32 ConstraintIndex = 0;
		for (int32 VertexIndex = 0; VertexIndex < Particles.GetRangeSize(); ++VertexIndex)
		{
			if (HasPlanarData[VertexIndex])
			{
				PlanarConstraint.GetUniqueConstraintIndices()[ConstraintIndex] = VertexIndex;
				PlanarConstraint.GetTargetPositions()[ConstraintIndex] = PlanarDataPositions[VertexIndex];
				PlanarConstraint.GetTargetNormals()[ConstraintIndex] = PlanarDataNormals[VertexIndex];
				PlanarConstraint.GetTargetVelocities()[ConstraintIndex] = PlanarDataVelocities.IsValidIndex(VertexIndex) ? PlanarDataVelocities[VertexIndex] : FSolverVec3(0.f);
				++ConstraintIndex;
			}
		}
		check(ConstraintIndex == NumPlanarConstraints);
	}
}

void FPBDSoftBodyCollisionConstraintBase::ApplyPlanarConstraints(FSolverParticlesRange& Particles, const FSolverReal Dt)
{
	PlanarConstraint.Apply(Particles, Dt);
}

template<bool bLockAndWriteContacts, bool bWithFriction>
void FPBDSoftBodyCollisionConstraintBase::ApplyInternalCCD(FSolverParticlesRange& Particles, const FSolverReal Dt, const TArray<FSolverCollisionParticlesRange>& CollisionParticles) const
{
	if (!bEnableSimpleColliders && !bEnableComplexColliders)
	{
		return;
	}

	FPAndInvM* const PAndInvM = Particles.GetPAndInvM().GetData();
	const FSolverVec3* const X = Particles.XArray().GetData();
	PhysicsParallelFor(Particles.GetRangeSize(), [this, Dt, &PAndInvM, &X, &CollisionParticles](int32 Index)
	{
		if (PAndInvM[Index].InvM == (FSolverReal)0.)
		{
			return;
		}

		for (const FSolverCollisionParticlesRange& CollisionParticlesRange : CollisionParticles)
		{
			const TConstArrayView<FSolverRigidTransform3> CollisionTransforms = CollisionParticlesRange.GetConstArrayView(LastCollisionTransforms);

			for (int32 CollisionIndex = 0; CollisionIndex < CollisionParticlesRange.GetRangeSize(); ++CollisionIndex)
			{
				const FImplicitObjectPtr& Geometry = CollisionParticlesRange.GetGeometry(CollisionIndex);
				const Chaos::EImplicitObjectType CollisionType = Geometry->GetType();
				if (Private::IsComplexBatchCollider(CollisionType))
				{
					if (!bEnableComplexColliders)
					{
						continue;
					}
				}
				else if (Private::IsSimpleCollider(CollisionType))
				{
					if (!bEnableSimpleColliders)
					{
						continue;
					}
				}
				else
				{
					continue;
				}
				const FSolverRigidTransform3 Frame(CollisionParticlesRange.GetX(CollisionIndex), CollisionParticlesRange.R(CollisionIndex));

				const Pair<FVec3, bool> PointPair = CollisionParticlesRange.GetGeometry(CollisionIndex)->FindClosestIntersection(  // Geometry operates in FReal
					FVec3(CollisionTransforms[CollisionIndex].InverseTransformPositionNoScale(X[Index])),        // hence the back and forth
					FVec3(Frame.InverseTransformPositionNoScale(PAndInvM[Index].P)), (FReal)(SoftBodyCollisionThicknessWeighted.GetValue(Index) + CollisionThickness));  // FVec3/FReal conversions

				if (PointPair.Second)
				{
					const FSolverVec3 Normal = FSolverVec3(CollisionParticlesRange.GetGeometry(CollisionIndex)->Normal(PointPair.First));
					const FSolverVec3 NormalWorld = Frame.TransformVectorNoScale(Normal);
					const FSolverVec3 ContactWorld = Frame.TransformPositionNoScale(UE::Math::TVector<FSolverReal>(PointPair.First));

					if constexpr (bLockAndWriteContacts)
					{
						checkSlow(Contacts);
						checkSlow(Normals);
						FScopeLock Lock(&DebugMutex);
						CollisionParticlesRange.GetArrayView(*CollisionParticleCollided)[CollisionIndex] = true;
						Contacts->Emplace(ContactWorld);
						Normals->Emplace(NormalWorld);
					}
					const FSolverVec3 Direction = ContactWorld - PAndInvM[Index].P;
					const FSolverReal Penetration = FMath::Max((FSolverReal)0., FSolverVec3::DotProduct(NormalWorld, Direction)) + (FSolverReal)UE_THRESH_POINT_ON_PLANE;

					PAndInvM[Index].P += Penetration * NormalWorld;
					
					if constexpr (bWithFriction)
					{
						// Friction
						FSolverVec3 CollisionV = CollisionParticlesRange.V(CollisionIndex);
						FSolverVec3 CollisionX = CollisionParticlesRange.X(CollisionIndex);
						FSolverVec3 CollisionW = CollisionParticlesRange.W(CollisionIndex);
						if (const TWeightedLatticeImplicitObject<FLevelSet>* LevelSet = CollisionParticlesRange.GetGeometry(CollisionIndex)->GetObject< TWeightedLatticeImplicitObject<FLevelSet> >())
						{
							TArray<FWeightedLatticeImplicitObject::FEmbeddingCoordinate> Coordinates;
							LevelSet->GetEmbeddingCoordinates(PointPair.First, Coordinates, false);
							int32 ClosestCoordIndex = INDEX_NONE;
							double ClosestCoordPhi = UE_BIG_NUMBER;
							for (int32 CoordIndex = 0; CoordIndex < Coordinates.Num(); ++CoordIndex)
							{
								FVec3 NormalUnused;
								const double CoordPhi = FMath::Abs(LevelSet->GetEmbeddedObject()->PhiWithNormal(Coordinates[CoordIndex].UndeformedPosition(LevelSet->GetGrid()), NormalUnused));
								if (CoordPhi < ClosestCoordPhi)
								{
									ClosestCoordIndex = CoordIndex;
									ClosestCoordPhi = CoordPhi;
								}
							}
							if (ClosestCoordIndex != INDEX_NONE)
							{
								if (const FPBDComplexColliderBoneData* const ColliderBoneData = ComplexBoneData.Find(FParticleRangeIndex(CollisionParticlesRange.GetRangeId(), CollisionIndex)))
								{
									const int32 StrongestBone = Coordinates[ClosestCoordIndex].GreatestInfluenceBone(LevelSet->GetBoneData());
									if (ColliderBoneData->MappedBoneIndices.IsValidIndex(StrongestBone))
									{
										const int32 MappedIndex = ColliderBoneData->MappedBoneIndices[StrongestBone];
										CollisionV = ColliderBoneData->V[MappedIndex];
										CollisionX = ColliderBoneData->X[MappedIndex];
										CollisionW = ColliderBoneData->W[MappedIndex];
									}
								}
							}
						}

						Private::ApplyFriction(PAndInvM[Index].P, X[Index], NormalWorld, Penetration * FrictionCoefficient, Dt, CollisionX, CollisionV, CollisionW);
					}
				}
			}
		}
	});
}

#if INTEL_ISPC
void FPBDSoftBodyCollisionConstraintBase::ApplySimpleInternalISPC(FSolverParticlesRange& Particles, const FSolverReal Dt, const TArray<FSolverCollisionParticlesRange>& CollisionParticles, bool bGeneratePlanarConstraints)
{
	if (!bEnableSimpleColliders)
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FPBDSoftBodyCollisionConstraint_ApplySimpleInternalISPC);
	SCOPE_CYCLE_COUNTER(STAT_ChaosPBDCollisionConstraintApply);

	check(bRealTypeCompatibleWithISPC);

	const bool bWithFriction = FrictionCoefficient > (FSolverReal)UE_KINDA_SMALL_NUMBER;

	const int32 NumBatches = FMath::CeilToInt((FSolverReal)(Particles.GetRangeSize()) / (FSolverReal)Chaos_SoftBodyCollision_ISPC_ParallelBatchSize);

	// Simple colliders
	PhysicsParallelFor(NumBatches, [this, &Particles, Dt, &CollisionParticles, bWithFriction, bGeneratePlanarConstraints](int32 BatchNumber)
		{
			const int32 BatchBegin = (Chaos_SoftBodyCollision_ISPC_ParallelBatchSize * BatchNumber);
			const int32 BatchEnd = FMath::Min(Particles.GetRangeSize(), BatchBegin + Chaos_SoftBodyCollision_ISPC_ParallelBatchSize);

			for (const FSolverCollisionParticlesRange& CollisionParticlesRange : CollisionParticles)
			{
				if (bGeneratePlanarConstraints)
				{
					if (bWithFriction)
					{
						ispc::ApplyPerParticleSimpleCollisionFastFrictionAndGeneratePlanarConstraints(
							(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
							HasPlanarData.GetData(),
							(ispc::FVector3f*)PlanarDataPositions.GetData(),
							(ispc::FVector3f*)PlanarDataNormals.GetData(),
							(ispc::FVector3f*)PlanarDataVelocities.GetData(),
							(const ispc::FVector3f*)Particles.XArray().GetData(),
							(const ispc::FVector3f*)CollisionParticlesRange.GetV().GetData(),
							(const ispc::FVector3f*)CollisionParticlesRange.XArray().GetData(),
							(const ispc::FVector3f*)CollisionParticlesRange.GetW().GetData(),
							(const ispc::FVector4f*)CollisionParticlesRange.GetR().GetData(),
							FrictionCoefficient,
							CollisionThickness,
							reinterpret_cast<const ispc::FVector2f&>(SoftBodyCollisionThicknessWeighted.GetOffsetRange()),
							SoftBodyCollisionThicknessWeighted.GetMapValues().GetData(),
							(const uint8*)&CollisionParticlesRange,
							(const uint8*)CollisionParticlesRange.GetAllGeometry().GetData(),
							sizeof(FImplicitObject),
							FImplicitObject::GetOffsetOfType(),
							FImplicitObject::GetOffsetOfMargin(),
							Dt,
							CollisionParticlesRange.GetRangeSize(),
							BatchBegin,
							BatchEnd);
					}
					else
					{
						ispc::ApplyPerParticleSimpleCollisionNoFrictionAndGeneratePlanarConstraints(
							(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
							HasPlanarData.GetData(),
							(ispc::FVector3f*)PlanarDataPositions.GetData(),
							(ispc::FVector3f*)PlanarDataNormals.GetData(),
							(const ispc::FVector3f*)Particles.XArray().GetData(),
							(const ispc::FVector3f*)CollisionParticlesRange.XArray().GetData(),
							(const ispc::FVector4f*)CollisionParticlesRange.GetR().GetData(),
							CollisionThickness,
							reinterpret_cast<const ispc::FVector2f&>(SoftBodyCollisionThicknessWeighted.GetOffsetRange()),
							SoftBodyCollisionThicknessWeighted.GetMapValues().GetData(),
							(const uint8*)&CollisionParticlesRange,
							(const uint8*)CollisionParticlesRange.GetAllGeometry().GetData(),
							sizeof(FImplicitObject),
							FImplicitObject::GetOffsetOfType(),
							FImplicitObject::GetOffsetOfMargin(),
							CollisionParticlesRange.GetRangeSize(),
							BatchBegin,
							BatchEnd);
					}
				}
				else
				{
					const bool bHasInnerCollisionThickness = (InnerCollisionThickness.GetWeightedValue() != FSolverVec2::ZeroVector);
					const bool bHasInnerCollisionThicknessWeightMap = bHasInnerCollisionThickness && InnerCollisionThickness.HasWeightMap();

					const TConstArrayView<FSolverRigidTransform3> CollisionTransforms = CollisionParticlesRange.GetConstArrayView(LastCollisionTransforms);
					if (bWithFriction)
					{
						ispc::ApplyPerParticleSimpleCollisionFastFriction(
							(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
							(const ispc::FVector3f*)Particles.XArray().GetData(),
							(const ispc::FVector3f*)CollisionParticlesRange.GetV().GetData(),
							(const ispc::FVector3f*)CollisionParticlesRange.XArray().GetData(),
							(const ispc::FVector3f*)CollisionParticlesRange.GetW().GetData(),
							(const ispc::FVector4f*)CollisionParticlesRange.GetR().GetData(),
							(const ispc::FTransform3f*)CollisionTransforms.GetData(),
							FrictionCoefficient,
							CollisionThickness,
							reinterpret_cast<const ispc::FVector2f&>(SoftBodyCollisionThicknessWeighted.GetOffsetRange()),
							SoftBodyCollisionThicknessWeighted.GetMapValues().GetData(),
							bHasInnerCollisionThickness ? InnerCollisionThickness.GetTable().GetData() : nullptr,
							bHasInnerCollisionThicknessWeightMap ? InnerCollisionThickness.GetIndices().GetData() : nullptr,
							(const uint8*)&CollisionParticlesRange,
							(const uint8*)CollisionParticlesRange.GetAllGeometry().GetData(),
							sizeof(FImplicitObject),
							FImplicitObject::GetOffsetOfType(),
							FImplicitObject::GetOffsetOfMargin(),
							Dt,
							CollisionParticlesRange.GetRangeSize(),
							BatchBegin,
							BatchEnd);
					}
					else
					{
						ispc::ApplyPerParticleSimpleCollisionNoFriction(
							(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
							(const ispc::FVector3f*)Particles.XArray().GetData(),
							(const ispc::FVector3f*)CollisionParticlesRange.XArray().GetData(),
							(const ispc::FVector4f*)CollisionParticlesRange.GetR().GetData(),
							(const ispc::FTransform3f*)CollisionTransforms.GetData(),
							CollisionThickness,
							reinterpret_cast<const ispc::FVector2f&>(SoftBodyCollisionThicknessWeighted.GetOffsetRange()),
							SoftBodyCollisionThicknessWeighted.GetMapValues().GetData(),
							bHasInnerCollisionThickness ? InnerCollisionThickness.GetTable().GetData() : nullptr,
							bHasInnerCollisionThicknessWeightMap ? InnerCollisionThickness.GetIndices().GetData() : nullptr,
							(const uint8*)&CollisionParticlesRange,
							(const uint8*)CollisionParticlesRange.GetAllGeometry().GetData(),
							sizeof(FImplicitObject),
							FImplicitObject::GetOffsetOfType(),
							FImplicitObject::GetOffsetOfMargin(),
							CollisionParticlesRange.GetRangeSize(),
							BatchBegin,
							BatchEnd);
					}
				}
			}
		});
}

void FPBDSoftBodyCollisionConstraintBase::ApplyComplexInternalISPC(FSolverParticlesRange& Particles, const FSolverReal Dt, const TArray<FSolverCollisionParticlesRange>& CollisionParticles, bool bGeneratePlanarConstraints)
{
	if (!bEnableComplexColliders)
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FPBDSoftBodyCollisionConstraint_ApplyComplexInternalISPC);

	const bool bWithFriction = FrictionCoefficient > (FSolverReal)UE_KINDA_SMALL_NUMBER;
	for (const FSolverCollisionParticlesRange& CollisionParticlesRange : CollisionParticles)
	{
		for (int32 CollisionIndex = 0; CollisionIndex < CollisionParticlesRange.GetRangeSize(); ++CollisionIndex)
		{
			const FImplicitObjectPtr& Geometry = CollisionParticlesRange.GetGeometry(CollisionIndex);
			const Chaos::EImplicitObjectType CollisionType = Geometry->GetType();
			if (!Private::IsComplexBatchCollider(CollisionType))
			{
				continue;
			}

			const FSolverVec3& CollisionX = CollisionParticlesRange.X(CollisionIndex);
			const FSolverRotation3& CollisionR = CollisionParticlesRange.R(CollisionIndex);
			const FSolverRigidTransform3 Frame(CollisionX, CollisionR);
			if (bWithFriction)
			{
				const FSolverVec3& CollisionV = CollisionParticlesRange.V(CollisionIndex);
				const FSolverVec3& CollisionW = CollisionParticlesRange.W(CollisionIndex);

				if (const FMLLevelSet* const MLLevelSet = Geometry->GetObject<FMLLevelSet>())
				{
					BatchPhis.SetNumUninitialized(Particles.GetRangeSize());
					BatchNormals.SetNumUninitialized(Particles.GetRangeSize());

					// Batch Query
					constexpr int32 MLLevelSetThreadNum = 0;
					MLLevelSet->BatchPhiWithNormal(Particles.GetPAndInvM(), Frame, BatchPhis, BatchNormals, FMath::Max(SoftBodyCollisionThicknessWeighted.GetLow(), SoftBodyCollisionThicknessWeighted.GetHigh()) + CollisionThickness, MLLevelSetThreadNum, 0, Particles.GetRangeSize());

					// Apply
					if (bGeneratePlanarConstraints)
					{
						ispc::ApplyPerParticleBatchCollisionFastFrictionAndGeneratePlanarConstraints(
							(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
							HasPlanarData.GetData(),
							(ispc::FVector3f*)PlanarDataPositions.GetData(),
							(ispc::FVector3f*)PlanarDataNormals.GetData(),
							(ispc::FVector3f*)PlanarDataVelocities.GetData(),
							(const ispc::FVector3f*)Particles.XArray().GetData(),
							BatchPhis.GetData(),
							(const ispc::FVector3f*)BatchNormals.GetData(),
							reinterpret_cast<const ispc::FVector3f&>(CollisionV),
							reinterpret_cast<const ispc::FVector3f&>(CollisionX),
							reinterpret_cast<const ispc::FVector3f&>(CollisionW),
							reinterpret_cast<const ispc::FVector4f&>(CollisionR),
							FrictionCoefficient,
							CollisionThickness,
							reinterpret_cast<const ispc::FVector2f&>(SoftBodyCollisionThicknessWeighted.GetOffsetRange()),
							SoftBodyCollisionThicknessWeighted.GetMapValues().GetData(),
							Dt,
							0,
							Particles.GetRangeSize());
					}
					else
					{
						ispc::ApplyPerParticleBatchCollisionFastFriction(
							(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
							(const ispc::FVector3f*)Particles.XArray().GetData(),
							BatchPhis.GetData(),
							(const ispc::FVector3f*)BatchNormals.GetData(),
							reinterpret_cast<const ispc::FVector3f&>(CollisionV),
							reinterpret_cast<const ispc::FVector3f&>(CollisionX),
							reinterpret_cast<const ispc::FVector3f&>(CollisionW),
							reinterpret_cast<const ispc::FVector4f&>(CollisionR),
							FrictionCoefficient,
							CollisionThickness,
							reinterpret_cast<const ispc::FVector2f&>(SoftBodyCollisionThicknessWeighted.GetOffsetRange()),
							SoftBodyCollisionThicknessWeighted.GetMapValues().GetData(),
							Dt,
							0,
							Particles.GetRangeSize());
					}
				}
				else if (const TWeightedLatticeImplicitObject<FLevelSet>* const LevelSet = Geometry->GetObject<TWeightedLatticeImplicitObject<FLevelSet>>())
				{
					if (const FPBDComplexColliderBoneData* const ColliderBoneData = ComplexBoneData.Find(FParticleRangeIndex(CollisionParticlesRange.GetRangeId(), CollisionIndex)))
					{
						LevelSet->BatchPhiWithNormalAndGreatestInfluenceBone(Particles.GetPAndInvM(), Frame, FMath::Max(SoftBodyCollisionThicknessWeighted.GetLow(), SoftBodyCollisionThicknessWeighted.GetHigh()) + CollisionThickness, BatchPhis, BatchNormals, BatchVelocityBones);

						if (bGeneratePlanarConstraints)
						{
							ispc::ApplyPerParticleBatchCollisionFastFrictionWithVelocityBonesAndGeneratePlanarConstraints(
								(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
								HasPlanarData.GetData(),
								(ispc::FVector3f*)PlanarDataPositions.GetData(),
								(ispc::FVector3f*)PlanarDataNormals.GetData(),
								(ispc::FVector3f*)PlanarDataVelocities.GetData(),
								(const ispc::FVector3f*)Particles.XArray().GetData(),
								BatchPhis.GetData(),
								(const ispc::FVector3f*)BatchNormals.GetData(),
								BatchVelocityBones.GetData(),
								reinterpret_cast<const ispc::FVector3f&>(CollisionV),
								reinterpret_cast<const ispc::FVector3f&>(CollisionX),
								reinterpret_cast<const ispc::FVector3f&>(CollisionW),
								reinterpret_cast<const ispc::FVector4f&>(CollisionR),
								ColliderBoneData->MappedBoneIndices.GetData(),
								ColliderBoneData->MappedBoneIndices.Num(),
								(const ispc::FVector3f*)ColliderBoneData->V.GetData(),
								(const ispc::FVector3f*)ColliderBoneData->X.GetData(),
								(const ispc::FVector3f*)ColliderBoneData->W.GetData(),
								FrictionCoefficient,
								CollisionThickness,
								reinterpret_cast<const ispc::FVector2f&>(SoftBodyCollisionThicknessWeighted.GetOffsetRange()),
								SoftBodyCollisionThicknessWeighted.GetMapValues().GetData(),
								Dt,
								0,
								Particles.GetRangeSize());
						}
						else
						{
							ispc::ApplyPerParticleBatchCollisionFastFrictionWithVelocityBones(
								(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
								(const ispc::FVector3f*)Particles.XArray().GetData(),
								BatchPhis.GetData(),
								(const ispc::FVector3f*)BatchNormals.GetData(),
								BatchVelocityBones.GetData(),
								reinterpret_cast<const ispc::FVector3f&>(CollisionV),
								reinterpret_cast<const ispc::FVector3f&>(CollisionX),
								reinterpret_cast<const ispc::FVector3f&>(CollisionW),
								reinterpret_cast<const ispc::FVector4f&>(CollisionR),
								ColliderBoneData->MappedBoneIndices.GetData(),
								ColliderBoneData->MappedBoneIndices.Num(),
								(const ispc::FVector3f*)ColliderBoneData->V.GetData(),
								(const ispc::FVector3f*)ColliderBoneData->X.GetData(),
								(const ispc::FVector3f*)ColliderBoneData->W.GetData(),
								FrictionCoefficient,
								CollisionThickness,
								reinterpret_cast<const ispc::FVector2f&>(SoftBodyCollisionThicknessWeighted.GetOffsetRange()),
								SoftBodyCollisionThicknessWeighted.GetMapValues().GetData(),
								Dt,
								0,
								Particles.GetRangeSize());
						}
					}
					else
					{
						LevelSet->BatchPhiWithNormal(Particles.GetPAndInvM(), Frame, BatchPhis, BatchNormals);
						if (bGeneratePlanarConstraints)
						{
							ispc::ApplyPerParticleBatchCollisionFastFrictionAndGeneratePlanarConstraints(
								(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
								HasPlanarData.GetData(),
								(ispc::FVector3f*)PlanarDataPositions.GetData(),
								(ispc::FVector3f*)PlanarDataNormals.GetData(),
								(ispc::FVector3f*)PlanarDataVelocities.GetData(),
								(const ispc::FVector3f*)Particles.XArray().GetData(),
								BatchPhis.GetData(),
								(const ispc::FVector3f*)BatchNormals.GetData(),
								reinterpret_cast<const ispc::FVector3f&>(CollisionV),
								reinterpret_cast<const ispc::FVector3f&>(CollisionX),
								reinterpret_cast<const ispc::FVector3f&>(CollisionW),
								reinterpret_cast<const ispc::FVector4f&>(CollisionR),
								FrictionCoefficient,
								CollisionThickness,
								reinterpret_cast<const ispc::FVector2f&>(SoftBodyCollisionThicknessWeighted.GetOffsetRange()),
								SoftBodyCollisionThicknessWeighted.GetMapValues().GetData(),
								Dt,
								0,
								Particles.GetRangeSize());
						}
						else
						{
							ispc::ApplyPerParticleBatchCollisionFastFriction(
								(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
								(const ispc::FVector3f*)Particles.XArray().GetData(),
								BatchPhis.GetData(),
								(const ispc::FVector3f*)BatchNormals.GetData(),
								reinterpret_cast<const ispc::FVector3f&>(CollisionV),
								reinterpret_cast<const ispc::FVector3f&>(CollisionX),
								reinterpret_cast<const ispc::FVector3f&>(CollisionW),
								reinterpret_cast<const ispc::FVector4f&>(CollisionR),
								FrictionCoefficient,
								CollisionThickness,
								reinterpret_cast<const ispc::FVector2f&>(SoftBodyCollisionThicknessWeighted.GetOffsetRange()),
								SoftBodyCollisionThicknessWeighted.GetMapValues().GetData(),
								Dt,
								0,
								Particles.GetRangeSize());
						}
					}
				}
				else
				{
					checkNoEntry();
				}
			}
			else
			{
				if (const FMLLevelSet* const MLLevelSet = Geometry->GetObject<FMLLevelSet>())
				{
					BatchPhis.SetNumUninitialized(Particles.GetRangeSize());
					BatchNormals.SetNumUninitialized(Particles.GetRangeSize());

					// Batch Query
					constexpr int32 MLLevelSetThreadNum = 0;
					MLLevelSet->BatchPhiWithNormal(Particles.GetPAndInvM(), Frame, BatchPhis, BatchNormals, FMath::Max(SoftBodyCollisionThicknessWeighted.GetLow(), SoftBodyCollisionThicknessWeighted.GetHigh()) + CollisionThickness, MLLevelSetThreadNum, 0, Particles.GetRangeSize());
				}
				else if (const TWeightedLatticeImplicitObject<FLevelSet>* const LevelSet = Geometry->GetObject<TWeightedLatticeImplicitObject<FLevelSet>>())
				{
					LevelSet->BatchPhiWithNormal(Particles.GetPAndInvM(), Frame, BatchPhis, BatchNormals);
				}
				else
				{
					checkNoEntry();
				}

				// Apply
				if (bGeneratePlanarConstraints)
				{
					ispc::ApplyPerParticleBatchCollisionNoFrictionAndGeneratePlanarConstraints(
						(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
						HasPlanarData.GetData(),
						(ispc::FVector3f*)PlanarDataPositions.GetData(),
						(ispc::FVector3f*)PlanarDataNormals.GetData(),
						(const ispc::FVector3f*)Particles.XArray().GetData(),
						BatchPhis.GetData(),
						(const ispc::FVector3f*)BatchNormals.GetData(),
						reinterpret_cast<const ispc::FVector4f&>(CollisionR),
						CollisionThickness,
						reinterpret_cast<const ispc::FVector2f&>(SoftBodyCollisionThicknessWeighted.GetOffsetRange()),
						SoftBodyCollisionThicknessWeighted.GetMapValues().GetData(),
						0,
						Particles.GetRangeSize());
				}
				else
				{
					ispc::ApplyPerParticleBatchCollisionNoFriction(
						(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
						(const ispc::FVector3f*)Particles.XArray().GetData(),
						BatchPhis.GetData(),
						(const ispc::FVector3f*)BatchNormals.GetData(),
						reinterpret_cast<const ispc::FVector4f&>(CollisionR),
						CollisionThickness,
						reinterpret_cast<const ispc::FVector2f&>(SoftBodyCollisionThicknessWeighted.GetOffsetRange()),
						SoftBodyCollisionThicknessWeighted.GetMapValues().GetData(),
						0,
						Particles.GetRangeSize());
				}
			}
		}
	}
}
#endif  // #if INTEL_ISPC

void FPBDSoftBodyCollisionConstraintBase::UpdateLinearSystem(const FSolverParticlesRange& Particles, const FSolverReal Dt, const TArray<FSolverCollisionParticlesRange>& CollisionParticles, FEvolutionLinearSystem& LinearSystem) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPBDSoftBodyCollisionConstraint_UpdateLinearSystem);

	if (CollisionParticles.IsEmpty() || ProximityStiffness == (FSolverReal)0.f)
	{
		return;
	}

	// Just going to allocate enough space for all possible collisions. 
	LinearSystem.ReserveForParallelAdd(Particles.GetRangeSize(), 0);

	// Just proximity forces for now
	const FPAndInvM* const PAndInvM = Particles.GetPAndInvM().GetData();
	const FSolverReal ClampedFriction = FMath::Clamp(FrictionCoefficient, (FSolverReal)0., (FSolverReal)1.);
	PhysicsParallelFor(Particles.GetRangeSize(), [this, Dt, ClampedFriction, &Particles, &PAndInvM, &CollisionParticles, &LinearSystem](int32 Index)
	{
		if (PAndInvM[Index].InvM == (FSolverReal)0.)
		{
			return;
		}

		bool bAddForce = false;
		FSolverVec3 Force(0.f);
		FSolverMatrix33 DfDx(0.f);
		for (const FSolverCollisionParticlesRange& CollisionParticlesRange : CollisionParticles)
		{
			for (int32 CollisionIndex = 0; CollisionIndex < CollisionParticlesRange.GetRangeSize(); ++CollisionIndex)
			{
				const FSolverRigidTransform3 Frame(CollisionParticlesRange.X(CollisionIndex), CollisionParticlesRange.R(CollisionIndex));
				const FVec3 RigidSpacePosition(Frame.InverseTransformPosition(PAndInvM[Index].P));  // PhiWithNormal requires FReal based arguments
				FVec3 ImplicitNormal;                                                                // since implicits don't use FSolverReal
				const FSolverReal Phi = (FSolverReal)CollisionParticlesRange.GetGeometry(CollisionIndex)->PhiWithNormal(RigidSpacePosition, ImplicitNormal);
				const FSolverReal Penetration = SoftBodyCollisionThicknessWeighted.GetValue(Index) + CollisionThickness - Phi; // This is related to the Normal impulse
				const FSolverVec3 Normal(ImplicitNormal);

				if (Penetration > (FSolverReal)0.)
				{
					bAddForce = true;

					const FSolverVec3 NormalWorld = Frame.TransformVector(Normal);

					// Repulsion force
					Force += ProximityStiffness * Penetration * NormalWorld;

					// Blend between a zero-length spring (stiction) and repulsion force based on friction
					// DfDx = -ProximityStiffness * ((1-FrictionCoefficient)*OuterProduct(N,N) + FrictionCoefficient * Identity)
					// Nothing here to match velocities... not sure if it's necessary, but this is a very stable force at least unlike any velocity-based thing.

					DfDx += -ProximityStiffness * (((FSolverReal)1. - ClampedFriction) * FSolverMatrix33::OuterProduct(NormalWorld, NormalWorld) + FSolverMatrix33(ClampedFriction, ClampedFriction, ClampedFriction));
				}
			}
		}

		if (bAddForce)
		{
			LinearSystem.AddForce(Particles, Force, Index, Dt);
			LinearSystem.AddSymmetricForceDerivative(Particles, &DfDx, nullptr, Index, Index, Dt);
		}
	});
}

FPBDSoftBodyCollisionConstraint::FPBDSoftBodyCollisionConstraint(
	const TArray<FSolverRigidTransform3>& InLastCollisionTransforms,
	const FCollectionPropertyConstFacade& PropertyCollection,
	FSolverReal InMeshScale,
	TArray<bool>* InCollisionParticleCollided,
	TArray<FSolverVec3>* InContacts,
	TArray<FSolverVec3>* InNormals,
	TArray<FSolverReal>* InPhis,
	const TMap<FParticleRangeIndex, FPBDComplexColliderBoneData>& InComplexBoneData,
	const TMap<FString, TConstArrayView<float>>& WeightMaps)
	: Base(InLastCollisionTransforms,
		InMeshScale* GetCollisionThickness(PropertyCollection, DefaultCollisionThickness),
		GetFrictionCoefficient(PropertyCollection, DefaultFrictionCoefficient),
		GetUseCCD(PropertyCollection, false),
		GetProximityStiffness(PropertyCollection, DefaultProximityStiffness),
		InCollisionParticleCollided, InContacts, InNormals, InPhis,
		GetWeightedFloatClothCollisionThickness(PropertyCollection,
			FSolverVec2(DefaultSoftBodyCollisionThickness, DefaultSoftBodyCollisionThickness)),
		GetEnableSimpleColliders(PropertyCollection, true),
		GetEnableComplexColliders(PropertyCollection, true),
		GetUsePlanarConstraintForSimpleColliders(PropertyCollection, bDefaultUsePlanarConstraintForSimpleColliders),
		GetUsePlanarConstraintForComplexColliders(PropertyCollection, bDefaultUsePlanarConstraintForComplexColliders),
		InComplexBoneData)
	, MeshScale(InMeshScale)
	, CollisionThicknessIndex(PropertyCollection)
	, ClothCollisionThicknessIndex(PropertyCollection)
	, FrictionCoefficientIndex(PropertyCollection)
	, UseCCDIndex(PropertyCollection)
	, ProximityStiffnessIndex(PropertyCollection)
	, EnableSimpleCollidersIndex(PropertyCollection)
	, UsePlanarConstraintForSimpleCollidersIndex(PropertyCollection)
	, EnableComplexCollidersIndex(PropertyCollection)
	, UsePlanarConstraintForComplexCollidersIndex(PropertyCollection)
{
	// Initialize the base class InnerCollisionThickness map
	const FSolverVec2 WeightedValue(GetWeightedFloatInnerCollisionThickness(PropertyCollection, FSolverVec2::ZeroVector));
	if (WeightedValue.X > (FSolverReal)0. || WeightedValue.Y > (FSolverReal)0.)
	{
		const FString WeighMapString = GetInnerCollisionThicknessString(PropertyCollection, InnerCollisionThicknessName.ToString());
		const TConstArrayView<FRealSingle>& WeightMap = WeightMaps.FindRef(WeighMapString);
		InnerCollisionThickness = FPBDWeightMap(
			WeightedValue * MeshScale,
			WeightMap,
			WeightMap.Num());
		InnerCollisionThickness.ApplyValues();
	}

	// Initialize the base class SoftBodyCollisionThicknessWeighted map
	{
		const FSolverVec2 SoftBodyWeightedValue(MeshScale * GetWeightedFloatClothCollisionThickness(PropertyCollection,
			FSolverVec2(DefaultSoftBodyCollisionThickness, DefaultSoftBodyCollisionThickness)));
		const FString SoftBodyWeightMapString = GetClothCollisionThicknessString(PropertyCollection, ClothCollisionThicknessName.ToString());
		const TConstArrayView<FRealSingle>& SoftBodyWeightMap = WeightMaps.FindRef(SoftBodyWeightMapString);
		SoftBodyCollisionThicknessWeighted = FPBDFlatWeightMap(
			SoftBodyWeightedValue, SoftBodyWeightMap, SoftBodyWeightMap.Num());
	}
}

void FPBDSoftBodyCollisionConstraint::SetProperties(
	const FCollectionPropertyConstFacade& PropertyCollection,
	const TMap<FString, TConstArrayView<float>>& WeightMaps)
{
	if (IsCollisionThicknessMutable(PropertyCollection))
	{
		CollisionThickness = MeshScale * GetCollisionThickness(PropertyCollection);
	}
	if (IsClothCollisionThicknessMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedValue(MeshScale * GetWeightedFloatClothCollisionThickness(PropertyCollection));
		if (IsClothCollisionThicknessStringDirty(PropertyCollection))
		{
			const FString& WeightMapName = GetClothCollisionThicknessString(PropertyCollection);
			const TConstArrayView<FRealSingle>& WeightMap = WeightMaps.FindRef(WeightMapName);
			SoftBodyCollisionThicknessWeighted = FPBDFlatWeightMap(WeightedValue, WeightMap, WeightMap.Num());
		}
		else
		{
			SoftBodyCollisionThicknessWeighted.SetWeightedValue(WeightedValue);
		}
	}
	if (IsFrictionCoefficientMutable(PropertyCollection))
	{
		FrictionCoefficient = GetFrictionCoefficient(PropertyCollection);
		PlanarConstraint.SetFrictionCoefficient(FrictionCoefficient);
	}
	if (IsUseCCDMutable(PropertyCollection))
	{
		bUseCCD = GetUseCCD(PropertyCollection);
	}
	if (IsProximityStiffnessMutable(PropertyCollection))
	{
		ProximityStiffness = GetProximityStiffness(PropertyCollection);
	}
	if (IsUsePlanarConstraintForSimpleCollidersMutable(PropertyCollection))
	{
		bUsePlanarConstraintForSimpleColliders = GetUsePlanarConstraintForSimpleColliders(PropertyCollection);
	}
	if (IsUsePlanarConstraintForComplexCollidersMutable(PropertyCollection))
	{
		bUsePlanarConstraintForComplexColliders = GetUsePlanarConstraintForComplexColliders(PropertyCollection);
	}
}

}  // End namespace Chaos::Softs
