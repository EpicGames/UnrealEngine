// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/PerParticlePBDCollisionConstraint.isph"

export void ApplyPerParticleBatchCollisionFastFriction(uniform FVector4f ParticlesPandInvM[],
													const uniform FVector3f ParticlesX[],
													const uniform float BatchPhis[],
													const uniform FVector3f BatchNormals[],
													const uniform FVector3f& CollisionV,
													const uniform FVector3f& CollisionX,
													const uniform FVector3f& CollisionW,
													const uniform FVector4f& CollisionR,
													const uniform float PerGroupFriction,
													const uniform float PerGroupCollisionThickness,
													const uniform FVector2f& SoftBodyThicknessOffsetRange,
													const uniform float SoftBodyThicknessMap[],
													const uniform float Dt,
													const uniform int OuterParticleBatchBegin,
													const uniform int OuterParticleBatchEnd)
{
	foreach(Index = OuterParticleBatchBegin ... OuterParticleBatchEnd)
	{
		const FVector4f PandInvM = VectorLoad(&ParticlesPandInvM[extract(Index, 0)]);
		varying FVector3f P;
		varying float IM;
		UnzipPandInvM(PandInvM, P, IM);

		if(all(IM == 0))
		{
			continue;
		}

		const float Phi = BatchPhis[Index];
		
		varying float CollisionThickness = PerGroupCollisionThickness + SoftBodyThicknessOffsetRange.V[0];
		if (SoftBodyThicknessMap != NULL)
		{
			CollisionThickness += SoftBodyThicknessMap[Index] * SoftBodyThicknessOffsetRange.V[1];
		}

		const float Penetration = CollisionThickness - Phi; // This is related to the Normal impulse
		if(IM != 0 && Penetration > 0.0f)
		{
			const FVector3f Normal = VectorLoad(&BatchNormals[extract(Index,0)]);

			// TransformVector. Scale is 1 so that part can be removed.
			const FVector3f NormalWorld = VectorQuaternionRotateVector(CollisionR, Normal);
			P = P + (Penetration * NormalWorld);
				
			const FVector3f X = VectorLoad(&ParticlesX[extract(Index, 0)]);
			ApplyFriction(P, X, NormalWorld, Penetration * PerGroupFriction, Dt, CollisionX, CollisionV, CollisionW);
			
			VectorStore(&ParticlesPandInvM[extract(Index, 0)], SetVector4(P, IM));
		}
	}
}

export void ApplyPerParticleBatchCollisionFastFrictionWithVelocityBones(uniform FVector4f ParticlesPandInvM[],
													const uniform FVector3f ParticlesX[],
													const uniform float BatchPhis[],
													const uniform FVector3f BatchNormals[],
													const uniform int BatchVelocityBones[],
													const uniform FVector3f& CollisionV,
													const uniform FVector3f& CollisionX,
													const uniform FVector3f& CollisionW,
													const uniform FVector4f& CollisionR,
													const uniform int MappedBoneIndices[],
													const uniform int NumMappedBones,
													const uniform FVector3f ColliderBoneV[],
													const uniform FVector3f ColliderBoneX[],
													const uniform FVector3f ColliderBoneW[],
													const uniform float PerGroupFriction,
													const uniform float PerGroupCollisionThickness,
													const uniform FVector2f& SoftBodyThicknessOffsetRange,
													const uniform float SoftBodyThicknessMap[],
													const uniform float Dt,
													const uniform int OuterParticleBatchBegin,
													const uniform int OuterParticleBatchEnd)
{
	foreach(Index = OuterParticleBatchBegin ... OuterParticleBatchEnd)
	{
		const FVector4f PandInvM = VectorLoad(&ParticlesPandInvM[extract(Index, 0)]);
		varying FVector3f P;
		varying float IM;
		UnzipPandInvM(PandInvM, P, IM);

		if(all(IM == 0))
		{
			continue;
		}

		const float Phi = BatchPhis[Index];
		
		varying float CollisionThickness = PerGroupCollisionThickness + SoftBodyThicknessOffsetRange.V[0];
		if (SoftBodyThicknessMap != NULL)
		{
			CollisionThickness += SoftBodyThicknessMap[Index] * SoftBodyThicknessOffsetRange.V[1];
		}

		const float Penetration = CollisionThickness - Phi; // This is related to the Normal impulse
		if(IM != 0 && Penetration > 0.0f)
		{
			const FVector3f Normal = VectorLoad(&BatchNormals[extract(Index,0)]);

			// TransformVector. Scale is 1 so that part can be removed.
			const FVector3f NormalWorld = VectorQuaternionRotateVector(CollisionR, Normal);
			P = P + (Penetration * NormalWorld);
				
			const FVector3f X = VectorLoad(&ParticlesX[extract(Index, 0)]);

			const int32 VelocityBone = BatchVelocityBones[Index];
			if(VelocityBone >= 0 && VelocityBone < NumMappedBones)
			{				
				#pragma ignore warning(perf)
				const int32 MappedIndex = MappedBoneIndices[VelocityBone];
				
				const FVector3f ColliderV = VectorGather(&ColliderBoneV[MappedIndex]);
				const FVector3f ColliderX = VectorGather(&ColliderBoneX[MappedIndex]);
				const FVector3f ColliderW = VectorGather(&ColliderBoneW[MappedIndex]);
				ApplyFriction(P, X, NormalWorld, Penetration * PerGroupFriction, Dt, ColliderX, ColliderV, ColliderW);
			}
			else
			{
				ApplyFriction(P, X, NormalWorld, Penetration * PerGroupFriction, Dt, CollisionX, CollisionV, CollisionW);
			}
			
			VectorStore(&ParticlesPandInvM[extract(Index, 0)], SetVector4(P, IM));
		}
	}
}

export void ApplyPerParticleBatchCollisionFastFrictionAndGeneratePlanarConstraints(uniform FVector4f ParticlesPandInvM[],
													uniform bool HasPlanarData[],
													uniform FVector3f PlanarDataPositions[],
													uniform FVector3f PlanarDataNormals[],
													uniform FVector3f PlanarDataVelocities[],
													const uniform FVector3f ParticlesX[],
													const uniform float BatchPhis[],
													const uniform FVector3f BatchNormals[],
													const uniform FVector3f& CollisionV,
													const uniform FVector3f& CollisionX,
													const uniform FVector3f& CollisionW,
													const uniform FVector4f& CollisionR,
													const uniform float PerGroupFriction,
													const uniform float PerGroupCollisionThickness,
													const uniform FVector2f& SoftBodyThicknessOffsetRange,
													const uniform float SoftBodyThicknessMap[],
													const uniform float Dt,
													const uniform int OuterParticleBatchBegin,
													const uniform int OuterParticleBatchEnd)
{
	foreach(Index = OuterParticleBatchBegin ... OuterParticleBatchEnd)
	{
		const FVector4f PandInvM = VectorLoad(&ParticlesPandInvM[extract(Index, 0)]);
		varying FVector3f P;
		varying float IM;
		UnzipPandInvM(PandInvM, P, IM);

		if(all(IM == 0))
		{
			continue;
		}

		const float Phi = BatchPhis[Index];
		
		varying float CollisionThickness = PerGroupCollisionThickness + SoftBodyThicknessOffsetRange.V[0];
		if (SoftBodyThicknessMap != NULL)
		{
			CollisionThickness += SoftBodyThicknessMap[Index] * SoftBodyThicknessOffsetRange.V[1];
		}

		const float Penetration = CollisionThickness - Phi; // This is related to the Normal impulse
		if(IM != 0 && Penetration > 0.0f)
		{
			const FVector3f Normal = VectorLoad(&BatchNormals[extract(Index,0)]);

			// TransformVector. Scale is 1 so that part can be removed.
			const FVector3f NormalWorld = VectorQuaternionRotateVector(CollisionR, Normal);
			P = P + (Penetration * NormalWorld);
			
			HasPlanarData[Index] = true;
			VectorStore(&PlanarDataPositions[extract(Index,0)], P);
			VectorStore(&PlanarDataNormals[extract(Index,0)], NormalWorld);

			const FVector3f X = VectorLoad(&ParticlesX[extract(Index, 0)]);
			FVector3f SurfaceVelocity;
			ApplyFriction(P, X, NormalWorld, Penetration * PerGroupFriction, Dt, CollisionX, CollisionV, CollisionW, SurfaceVelocity);

			VectorStore(&PlanarDataVelocities[extract(Index,0)], SurfaceVelocity);
			
			VectorStore(&ParticlesPandInvM[extract(Index, 0)], SetVector4(P, IM));
		}
	}
}

export void ApplyPerParticleBatchCollisionFastFrictionWithVelocityBonesAndGeneratePlanarConstraints(uniform FVector4f ParticlesPandInvM[],
													uniform bool HasPlanarData[],
													uniform FVector3f PlanarDataPositions[],
													uniform FVector3f PlanarDataNormals[],
													uniform FVector3f PlanarDataVelocities[],
													const uniform FVector3f ParticlesX[],
													const uniform float BatchPhis[],
													const uniform FVector3f BatchNormals[],
													const uniform int BatchVelocityBones[],
													const uniform FVector3f& CollisionV,
													const uniform FVector3f& CollisionX,
													const uniform FVector3f& CollisionW,
													const uniform FVector4f& CollisionR,
													const uniform int MappedBoneIndices[],
													const uniform int NumMappedBones,
													const uniform FVector3f ColliderBoneV[],
													const uniform FVector3f ColliderBoneX[],
													const uniform FVector3f ColliderBoneW[],
													const uniform float PerGroupFriction,
													const uniform float PerGroupCollisionThickness,
													const uniform FVector2f& SoftBodyThicknessOffsetRange,
													const uniform float SoftBodyThicknessMap[],
													const uniform float Dt,
													const uniform int OuterParticleBatchBegin,
													const uniform int OuterParticleBatchEnd)
{
	foreach(Index = OuterParticleBatchBegin ... OuterParticleBatchEnd)
	{
		const FVector4f PandInvM = VectorLoad(&ParticlesPandInvM[extract(Index, 0)]);
		varying FVector3f P;
		varying float IM;
		UnzipPandInvM(PandInvM, P, IM);

		if(all(IM == 0))
		{
			continue;
		}

		const float Phi = BatchPhis[Index];
		
		varying float CollisionThickness = PerGroupCollisionThickness + SoftBodyThicknessOffsetRange.V[0];
		if (SoftBodyThicknessMap != NULL)
		{
			CollisionThickness += SoftBodyThicknessMap[Index] * SoftBodyThicknessOffsetRange.V[1];
		}

		const float Penetration = CollisionThickness - Phi; // This is related to the Normal impulse
		if(IM != 0 && Penetration > 0.0f)
		{
			const FVector3f Normal = VectorLoad(&BatchNormals[extract(Index,0)]);

			// TransformVector. Scale is 1 so that part can be removed.
			const FVector3f NormalWorld = VectorQuaternionRotateVector(CollisionR, Normal);
			P = P + (Penetration * NormalWorld);
			
			HasPlanarData[Index] = true;
			VectorStore(&PlanarDataPositions[extract(Index,0)], P);
			VectorStore(&PlanarDataNormals[extract(Index,0)], NormalWorld);

			const FVector3f X = VectorLoad(&ParticlesX[extract(Index, 0)]);
			FVector3f SurfaceVelocity;
			const int32 VelocityBone = BatchVelocityBones[Index];
			if(VelocityBone >= 0 && VelocityBone < NumMappedBones)
			{				
				#pragma ignore warning(perf)
				const int32 MappedIndex = MappedBoneIndices[VelocityBone];
				
				const FVector3f ColliderV = VectorGather(&ColliderBoneV[MappedIndex]);
				const FVector3f ColliderX = VectorGather(&ColliderBoneX[MappedIndex]);
				const FVector3f ColliderW = VectorGather(&ColliderBoneW[MappedIndex]);
				ApplyFriction(P, X, NormalWorld, Penetration * PerGroupFriction, Dt, ColliderX, ColliderV, ColliderW, SurfaceVelocity);
			}
			else
			{
				ApplyFriction(P, X, NormalWorld, Penetration * PerGroupFriction, Dt, CollisionX, CollisionV, CollisionW, SurfaceVelocity);
			}

			VectorStore(&PlanarDataVelocities[extract(Index,0)], SurfaceVelocity);
			
			VectorStore(&ParticlesPandInvM[extract(Index, 0)], SetVector4(P, IM));
		}
	}
}


export void ApplyPerParticleBatchCollisionNoFriction(uniform FVector4f ParticlesPandInvM[],
													const uniform FVector3f ParticlesX[],
													const uniform float BatchPhis[],
													const uniform FVector3f BatchNormals[],
													const uniform FVector4f& CollisionR,
													const uniform float PerGroupCollisionThickness,
													const uniform FVector2f& SoftBodyThicknessOffsetRange,
													const uniform float SoftBodyThicknessMap[],
													const uniform int OuterParticleBatchBegin,
													const uniform int OuterParticleBatchEnd)
{
	foreach(Index = OuterParticleBatchBegin ... OuterParticleBatchEnd)
	{
		const FVector4f PandInvM = VectorLoad(&ParticlesPandInvM[extract(Index, 0)]);
		varying FVector3f P;
		varying float IM;
		UnzipPandInvM(PandInvM, P, IM);

		if(all(IM == 0))
		{
			continue;
		}

		const float Phi = BatchPhis[Index];
		
		varying float CollisionThickness = PerGroupCollisionThickness + SoftBodyThicknessOffsetRange.V[0];
		if (SoftBodyThicknessMap != NULL)
		{
			CollisionThickness += SoftBodyThicknessMap[Index] * SoftBodyThicknessOffsetRange.V[1];
		}

		const float Penetration = CollisionThickness - Phi; // This is related to the Normal impulse
		if(IM != 0 && Penetration > 0.0f)
		{
			const FVector3f Normal = VectorLoad(&BatchNormals[extract(Index,0)]);

			// TransformVector. Scale is 1 so that part can be removed.
			const FVector3f NormalWorld = VectorQuaternionRotateVector(CollisionR, Normal);
			P = P + (Penetration * NormalWorld);
			
			VectorStore(&ParticlesPandInvM[extract(Index, 0)], SetVector4(P, IM));
		}
	}
}

export void ApplyPerParticleBatchCollisionNoFrictionAndGeneratePlanarConstraints(uniform FVector4f ParticlesPandInvM[],
													uniform bool HasPlanarData[],
													uniform FVector3f PlanarDataPositions[],
													uniform FVector3f PlanarDataNormals[],
													const uniform FVector3f ParticlesX[],
													const uniform float BatchPhis[],
													const uniform FVector3f BatchNormals[],
													const uniform FVector4f& CollisionR,
													const uniform float PerGroupCollisionThickness,
													const uniform FVector2f& SoftBodyThicknessOffsetRange,
													const uniform float SoftBodyThicknessMap[],
													const uniform int OuterParticleBatchBegin,
													const uniform int OuterParticleBatchEnd)
{
	foreach(Index = OuterParticleBatchBegin ... OuterParticleBatchEnd)
	{
		const FVector4f PandInvM = VectorLoad(&ParticlesPandInvM[extract(Index, 0)]);
		varying FVector3f P;
		varying float IM;
		UnzipPandInvM(PandInvM, P, IM);

		if(all(IM == 0))
		{
			continue;
		}

		const float Phi = BatchPhis[Index];
		
		varying float CollisionThickness = PerGroupCollisionThickness + SoftBodyThicknessOffsetRange.V[0];
		if (SoftBodyThicknessMap != NULL)
		{
			CollisionThickness += SoftBodyThicknessMap[Index] * SoftBodyThicknessOffsetRange.V[1];
		}

		const float Penetration = CollisionThickness - Phi; // This is related to the Normal impulse
		if(IM != 0 && Penetration > 0.0f)
		{
			const FVector3f Normal = VectorLoad(&BatchNormals[extract(Index,0)]);

			// TransformVector. Scale is 1 so that part can be removed.
			const FVector3f NormalWorld = VectorQuaternionRotateVector(CollisionR, Normal);
			P = P + (Penetration * NormalWorld);
			
			HasPlanarData[Index] = true;
			VectorStore(&PlanarDataPositions[extract(Index,0)], P);
			VectorStore(&PlanarDataNormals[extract(Index,0)], NormalWorld);
			
			VectorStore(&ParticlesPandInvM[extract(Index, 0)], SetVector4(P, IM));
		}
	}
}
