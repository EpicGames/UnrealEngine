// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/PerParticlePBDCollisionConstraint.isph"


extern "C" unmasked void ApplySimpleCCDNoFriction(
	const uniform uint8* uniform CollisionParticlesRange,
	const uniform int CollisionIndex,
	const uniform FTransform3f& LastCollisionTransform,
	const uniform float* uniform X,
	const uniform float* uniform Thickness,
	uniform float* uniform P,
	const uniform int ProgramCount, 
	const uniform int Mask);

extern "C" unmasked void ApplySimpleCCDWithFriction(
	const uniform uint8* uniform CollisionParticlesRange,
	const uniform int CollisionIndex,
	const uniform FTransform3f& LastCollisionTransform,
	const uniform float* uniform X,
	const uniform float* uniform Thickness,
	const uniform float uniform FrictionCoefficient,
	const uniform float Dt,
	uniform float* uniform P,
	const uniform int ProgramCount, 
	const uniform int Mask);

extern "C" unmasked void GetPhiWithNormalCollisionParticleRange(const uniform uint8 *uniform CollisionParticlesRange, const uniform float *uniform V, uniform float *uniform Normal, uniform float *uniform Phi, const uniform int i, const uniform int ProgramCount, const uniform int Mask);

export void ApplyPerParticleSimpleCollisionFastFriction(uniform FVector4f ParticlesPandInvM[],
													const uniform FVector3f ParticlesX[],
													const uniform FVector3f CollisionV[],
													const uniform FVector3f CollisionX[],
													const uniform FVector3f CollisionW[],
													const uniform FVector4f CollisionR[],
													const uniform FTransform3f LastCollisionTransforms[],
													const uniform float PerGroupFriction,
													const uniform float PerGroupCollisionThickness,
													const uniform FVector2f& SoftBodyThicknessOffsetRange,
													const uniform float SoftBodyThicknessMap[],
													const uniform float *uniform InnerThicknessTable,
													const uniform uint8 *uniform InnerThicknessIndices,
													const uniform uint8 *uniform CollisionParticlesRange,
													const uniform uint8 *uniform Geometry,
													const uniform int SizeofFImplicitObject,
													const uniform int OffsetofGeometryType,
													const uniform int OffsetOfMargin,
													const uniform float Dt,
													const uniform int InnerCollisionCount,
													const uniform int OuterParticleBatchBegin,
													const uniform int OuterParticleBatchEnd)
{
	const uniform bool bHasInnerThickness = (InnerThicknessTable != NULL);
	if(bHasInnerThickness)
	{
		const uniform bool bHasInnerThicknessWeighmap = (InnerThicknessIndices != NULL);
		if(bHasInnerThicknessWeighmap)
		{
			foreach(Index = OuterParticleBatchBegin ... OuterParticleBatchEnd)
			{
				const FVector4f PandInvM = VectorLoad(&ParticlesPandInvM[extract(Index, 0)]);
				varying FVector3f P;
				varying float IM;
				UnzipPandInvM(PandInvM, P, IM);

				if(all(IM == 0))
				{
					continue;
				}

				varying float CollisionThickness = PerGroupCollisionThickness + SoftBodyThicknessOffsetRange.V[0];
				if (SoftBodyThicknessMap != NULL)
				{
					CollisionThickness += SoftBodyThicknessMap[Index] * SoftBodyThicknessOffsetRange.V[1];
				}

				varying uint8 InnerThicknessIndex = InnerThicknessIndices[Index];
				#pragma ignore warning(perf)
				varying float InnerThickness = InnerThicknessTable[InnerThicknessIndex];

				for(uniform int i = 0; i < InnerCollisionCount; i++) // SequentialFor
				{
					const uniform SerializedPtr *uniform GeometryPtr = (const uniform SerializedPtr *uniform)Geometry;
					const uniform uint8 *uniform Ptr = GeometryPtr[i].Ptr;
					const uniform uint8 *uniform TypedPtr = Ptr + SizeofFImplicitObject;
					const uniform uint8 Type = Ptr[OffsetofGeometryType] & 0x3F;

					if(!IsSimpleCollider(Type))
					{
						continue;
					}

					// InverseTransformPosition. Can skip Scale calculation because it is 1 for xyz.
					const FVector3f TranslatedVec = P - CollisionX[i];
					const uniform FVector4f InverseR = QuatInverse(CollisionR[i]);
					const FVector3f PhiWithNormalInputFloat = VectorQuaternionRotateVector(InverseR, TranslatedVec);
					FVector3f Normal;
					float Penetration;
			
					uniform int IMMask = 0;
					if(IM != 0)
					{
						IMMask = lanemask();
					}

					if(!GetSimplePenetrationWithNormal(PhiWithNormalInputFloat, Type, Ptr, TypedPtr, SizeofFImplicitObject, OffsetofGeometryType, OffsetOfMargin, CollisionThickness, Penetration, Normal))
					{
						const uniform int Mask = IMMask & lanemask();
						if(Mask != 0)  // Do not call c++ when there is no work to do
						{
							float Phi = 0.f;
							GetPhiWithNormalCollisionParticleRange(CollisionParticlesRange, (const uniform float *uniform)&PhiWithNormalInputFloat, (uniform float *uniform)&Normal, (uniform float *uniform)&Phi, i, programCount, Mask);
							Penetration = CollisionThickness - Phi;
						}
					}

					const FVector3f X = VectorLoad(&ParticlesX[extract(Index, 0)]);

					if(Penetration > InnerThickness)
					{
						const uniform int Mask = IMMask & lanemask();
						if (Mask != 0)  // Do not call c++ when there is no work to do
						{
							ApplySimpleCCDWithFriction(
								CollisionParticlesRange,
								i,
								LastCollisionTransforms[i],
								(uniform float* uniform)&X,
								(uniform float* uniform)&CollisionThickness,
								PerGroupFriction,
								Dt,
								(uniform float *uniform)&P,
								programCount,
								Mask);
						}
					}
					else if(Penetration > 0.0f)
					{
						// TransformVector. Scale is 1 so that part can be removed.
						const FVector3f NormalWorld = VectorQuaternionRotateVector(CollisionR[i], Normal);
						P = P + (Penetration * NormalWorld);

						ApplyFriction(P, X, NormalWorld, Penetration * PerGroupFriction, Dt, CollisionX[i], CollisionV[i], CollisionW[i]);
					}
				}

				if(IM != 0)
				{
					VectorStore(&ParticlesPandInvM[extract(Index, 0)], SetVector4(P, IM));
				}
			}
		}
		else  // No weight map
		{
			uniform float InnerThickness = InnerThicknessTable[0];

			foreach(Index = OuterParticleBatchBegin ... OuterParticleBatchEnd)
			{
				const FVector4f PandInvM = VectorLoad(&ParticlesPandInvM[extract(Index, 0)]);
				varying FVector3f P;
				varying float IM;
				UnzipPandInvM(PandInvM, P, IM);

				if(all(IM == 0))
				{
					continue;
				}
				
				varying float CollisionThickness = PerGroupCollisionThickness + SoftBodyThicknessOffsetRange.V[0];
				if (SoftBodyThicknessMap != NULL)
				{
					CollisionThickness += SoftBodyThicknessMap[Index] * SoftBodyThicknessOffsetRange.V[1];
				}

				for(uniform int i = 0; i < InnerCollisionCount; i++) // SequentialFor
				{
					const uniform SerializedPtr *uniform GeometryPtr = (const uniform SerializedPtr *uniform)Geometry;
					const uniform uint8 *uniform Ptr = GeometryPtr[i].Ptr;
					const uniform uint8 *uniform TypedPtr = Ptr + SizeofFImplicitObject;
					const uniform uint8 Type = Ptr[OffsetofGeometryType] & 0x3F;

					if(!IsSimpleCollider(Type))
					{
						continue;
					}

					// InverseTransformPosition. Can skip Scale calculation because it is 1 for xyz.
					const FVector3f TranslatedVec = P - CollisionX[i];
					const uniform FVector4f InverseR = QuatInverse(CollisionR[i]);
					const FVector3f PhiWithNormalInputFloat = VectorQuaternionRotateVector(InverseR, TranslatedVec);
					FVector3f Normal;
					float Penetration;
			
					uniform int IMMask = 0;
					if(IM != 0)
					{
						IMMask = lanemask();
					}
					

					if(!GetSimplePenetrationWithNormal(PhiWithNormalInputFloat, Type, Ptr, TypedPtr, SizeofFImplicitObject, OffsetofGeometryType, OffsetOfMargin, CollisionThickness, Penetration, Normal))
					{
						const uniform int Mask = IMMask & lanemask();
						if(Mask != 0)  // Do not call c++ when there is no work to do
						{
							float Phi = 0.f;
							GetPhiWithNormalCollisionParticleRange(CollisionParticlesRange, (const uniform float *uniform)&PhiWithNormalInputFloat, (uniform float *uniform)&Normal, (uniform float *uniform)&Phi, i, programCount, Mask);
							Penetration = CollisionThickness - Phi;
						}
					}

					const FVector3f X = VectorLoad(&ParticlesX[extract(Index, 0)]);

					if(Penetration > InnerThickness)
					{
						const uniform int Mask = IMMask & lanemask();
						if (Mask != 0)  // Do not call c++ when there is no work to do
						{
							ApplySimpleCCDWithFriction(
								CollisionParticlesRange,
								i,
								LastCollisionTransforms[i],
								(uniform float* uniform)&X,
								(uniform float* uniform)&CollisionThickness,
								PerGroupFriction,
								Dt,
								(uniform float *uniform)&P,
								programCount,
								Mask);
						}
					}
					else if(Penetration > 0.0f)
					{
						// TransformVector. Scale is 1 so that part can be removed.
						const FVector3f NormalWorld = VectorQuaternionRotateVector(CollisionR[i], Normal);
						P = P + (Penetration * NormalWorld);

						ApplyFriction(P, X, NormalWorld, Penetration * PerGroupFriction, Dt, CollisionX[i], CollisionV[i], CollisionW[i]);
					}
				}

				if(IM != 0)
				{
					VectorStore(&ParticlesPandInvM[extract(Index, 0)], SetVector4(P, IM));
				}
			}
		}
	}
	else  // No inner thickness
	{
		foreach(Index = OuterParticleBatchBegin ... OuterParticleBatchEnd)
		{
			const FVector4f PandInvM = VectorLoad(&ParticlesPandInvM[extract(Index, 0)]);
			varying FVector3f P;
			varying float IM;
			UnzipPandInvM(PandInvM, P, IM);

			if(all(IM == 0))
			{
				continue;
			}
			
			varying float CollisionThickness = PerGroupCollisionThickness + SoftBodyThicknessOffsetRange.V[0];
			if (SoftBodyThicknessMap != NULL)
			{
				CollisionThickness += SoftBodyThicknessMap[Index] * SoftBodyThicknessOffsetRange.V[1];
			}

			for(uniform int i = 0; i < InnerCollisionCount; i++) // SequentialFor
			{
				const uniform SerializedPtr *uniform GeometryPtr = (const uniform SerializedPtr *uniform)Geometry;
				const uniform uint8 *uniform Ptr = GeometryPtr[i].Ptr;
				const uniform uint8 *uniform TypedPtr = Ptr + SizeofFImplicitObject;
				const uniform uint8 Type = Ptr[OffsetofGeometryType] & 0x3F;

				if(!IsSimpleCollider(Type))
				{
					continue;
				}

				// InverseTransformPosition. Can skip Scale calculation because it is 1 for xyz.
				const FVector3f TranslatedVec = P - CollisionX[i];
				const uniform FVector4f InverseR = QuatInverse(CollisionR[i]);
				const FVector3f PhiWithNormalInputFloat = VectorQuaternionRotateVector(InverseR, TranslatedVec);
				FVector3f Normal;
				float Penetration;	

				if(!GetSimplePenetrationWithNormal(PhiWithNormalInputFloat, Type, Ptr, TypedPtr, SizeofFImplicitObject, OffsetofGeometryType, OffsetOfMargin, CollisionThickness, Penetration, Normal))
				{
					uniform int IMMask = 0;
					if(IM != 0)
					{
						IMMask = lanemask();
					}
					if(IMMask != 0)  // Do not call c++ when there is no work to do
					{
						float Phi = 0.f;
						GetPhiWithNormalCollisionParticleRange(CollisionParticlesRange, (const uniform float *uniform)&PhiWithNormalInputFloat, (uniform float *uniform)&Normal, (uniform float *uniform)&Phi, i, programCount, IMMask);
						Penetration = CollisionThickness - Phi;
					}
				}

				if(Penetration > 0.0f)
				{
					// TransformVector. Scale is 1 so that part can be removed.
					const FVector3f NormalWorld = VectorQuaternionRotateVector(CollisionR[i], Normal);
					P = P + (Penetration * NormalWorld);

					const FVector3f X = VectorLoad(&ParticlesX[extract(Index, 0)]);
					ApplyFriction(P, X, NormalWorld, Penetration * PerGroupFriction, Dt, CollisionX[i], CollisionV[i], CollisionW[i]);
				}
			}

			if(IM != 0)
			{
				VectorStore(&ParticlesPandInvM[extract(Index, 0)], SetVector4(P, IM));
			}
		}
	}
}

export void ApplyPerParticleSimpleCollisionFastFrictionAndGeneratePlanarConstraints(uniform FVector4f ParticlesPandInvM[],
													uniform bool HasPlanarData[],
													uniform FVector3f PlanarDataPositions[],
													uniform FVector3f PlanarDataNormals[],
													uniform FVector3f PlanarDataVelocities[],
													const uniform FVector3f ParticlesX[],
													const uniform FVector3f CollisionV[],
													const uniform FVector3f CollisionX[],
													const uniform FVector3f CollisionW[],
													const uniform FVector4f CollisionR[],
													const uniform float PerGroupFriction,
													const uniform float PerGroupCollisionThickness,
													const uniform FVector2f& SoftBodyThicknessOffsetRange,
													const uniform float SoftBodyThicknessMap[],
													const uniform uint8 *uniform CollisionParticlesRange,
													const uniform uint8 *uniform Geometry,
													const uniform int SizeofFImplicitObject,
													const uniform int OffsetofGeometryType,
													const uniform int OffsetOfMargin,
													const uniform float Dt,
													const uniform int InnerCollisionCount,
													const uniform int OuterParticleBatchBegin,
													const uniform int OuterParticleBatchEnd)
{
	foreach(Index = OuterParticleBatchBegin ... OuterParticleBatchEnd)
	{
		const FVector4f PandInvM = VectorLoad(&ParticlesPandInvM[extract(Index, 0)]);
		varying FVector3f P;
		varying float IM;
		UnzipPandInvM(PandInvM, P, IM);

		varying bool bHasPlanarData = false;
		varying FVector3f PlanarP;
		varying FVector3f PlanarN;
		varying FVector3f PlanarV;

		if(all(IM == 0))
		{
			continue;
		}

		varying float CollisionThickness = PerGroupCollisionThickness + SoftBodyThicknessOffsetRange.V[0];
		if (SoftBodyThicknessMap != NULL)
		{
			CollisionThickness += SoftBodyThicknessMap[Index] * SoftBodyThicknessOffsetRange.V[1];
		}

		for(uniform int i = 0; i < InnerCollisionCount; i++) // SequentialFor
		{
			const uniform SerializedPtr *uniform GeometryPtr = (const uniform SerializedPtr *uniform)Geometry;
			const uniform uint8 *uniform Ptr = GeometryPtr[i].Ptr;
			const uniform uint8 *uniform TypedPtr = Ptr + SizeofFImplicitObject;
			const uniform uint8 Type = Ptr[OffsetofGeometryType] & 0x3F;

			if(!IsSimpleCollider(Type))
			{
				continue;
			}

			// InverseTransformPosition. Can skip Scale calculation because it is 1 for xyz.
			const FVector3f TranslatedVec = P - CollisionX[i];
			const uniform FVector4f InverseR = QuatInverse(CollisionR[i]);
			const FVector3f PhiWithNormalInputFloat = VectorQuaternionRotateVector(InverseR, TranslatedVec);
			FVector3f Normal;
			float Penetration;
			
			if(!GetSimplePenetrationWithNormal(PhiWithNormalInputFloat, Type, Ptr, TypedPtr, SizeofFImplicitObject, OffsetofGeometryType, OffsetOfMargin, CollisionThickness, Penetration, Normal))
			{
				uniform int IMMask;
				if(IM != 0)
				{
					IMMask = lanemask();
				}
				
				float Phi;
				GetPhiWithNormalCollisionParticleRange(CollisionParticlesRange, (const uniform float *uniform)&PhiWithNormalInputFloat, (uniform float *uniform)&Normal, (uniform float *uniform)&Phi, i, programCount, IMMask);
				Penetration = CollisionThickness - Phi;
			}

			if(Penetration > 0.0f)
			{
				// TransformVector. Scale is 1 so that part can be removed.
				PlanarN = VectorQuaternionRotateVector(CollisionR[i], Normal);
				P = P + (Penetration * PlanarN);

				bHasPlanarData = true;
				PlanarP = P;
				const FVector3f X = VectorLoad(&ParticlesX[extract(Index, 0)]);
				ApplyFriction(P, X, PlanarN, Penetration * PerGroupFriction, Dt, CollisionX[i], CollisionV[i], CollisionW[i], PlanarV);
			}
		}

		if(IM != 0)
		{
			VectorStore(&ParticlesPandInvM[extract(Index, 0)], SetVector4(P, IM));

			if(bHasPlanarData)
			{
				HasPlanarData[Index] = true;
				VectorStore(&PlanarDataPositions[extract(Index,0)], PlanarP);
				VectorStore(&PlanarDataNormals[extract(Index,0)], PlanarN);
				VectorStore(&PlanarDataVelocities[extract(Index,0)], PlanarV);
			}
		}
	}
}

export void ApplyPerParticleSimpleCollisionNoFriction(uniform FVector4f ParticlesPandInvM[],
												const uniform FVector3f ParticlesX[],
												const uniform FVector3f CollisionX[],
												const uniform FVector4f CollisionR[],
												const uniform FTransform3f LastCollisionTransforms[],
												const uniform float PerGroupCollisionThickness,
												const uniform FVector2f& SoftBodyThicknessOffsetRange,
												const uniform float SoftBodyThicknessMap[],
												const uniform float *uniform InnerThicknessTable,
												const uniform uint8 *uniform InnerThicknessIndices,
												const uniform uint8 *uniform CollisionParticlesRange,
												const uniform uint8 *uniform Geometry,
												const uniform int SizeofFImplicitObject,
												const uniform int OffsetofGeometryType,
												const uniform int OffsetOfMargin,
												const uniform int InnerCollisionCount,
												const uniform int OuterParticleBatchBegin,
												const uniform int OuterParticleBatchEnd)
{
	const uniform bool bHasInnerThickness = (InnerThicknessTable != NULL);
	if(bHasInnerThickness)
	{
		const uniform bool bHasInnerThicknessWeighmap = (InnerThicknessIndices != NULL);
		if(bHasInnerThicknessWeighmap)
		{
			foreach(Index = OuterParticleBatchBegin ... OuterParticleBatchEnd)
			{
				const FVector4f PandInvM = VectorLoad(&ParticlesPandInvM[extract(Index, 0)]);
				varying FVector3f P;
				varying float IM;
				UnzipPandInvM(PandInvM, P, IM);

				if(all(IM == 0))
				{
					continue;
				}

				varying float CollisionThickness = PerGroupCollisionThickness + SoftBodyThicknessOffsetRange.V[0];
				if (SoftBodyThicknessMap != NULL)
				{
					CollisionThickness += SoftBodyThicknessMap[Index] * SoftBodyThicknessOffsetRange.V[1];
				}

				varying uint8 InnerThicknessIndex = InnerThicknessIndices[Index];
				#pragma ignore warning(perf)
				varying float InnerThickness = InnerThicknessTable[InnerThicknessIndex];

				for(uniform int i = 0; i < InnerCollisionCount; i++) // SequentialFor
				{
					const uniform SerializedPtr *uniform GeometryPtr = (const uniform SerializedPtr *uniform)Geometry;
					const uniform uint8 *uniform Ptr = GeometryPtr[i].Ptr;
					const uniform uint8 *uniform TypedPtr = Ptr + SizeofFImplicitObject;
					const uniform uint8 Type = Ptr[OffsetofGeometryType] & 0x3F;

					if(!IsSimpleCollider(Type))
					{
						continue;
					}

					// InverseTransformPosition. Can skip Scale calculation because it is 1 for xyz.
					const FVector3f TranslatedVec = P - CollisionX[i];
					const uniform FVector4f InverseR = QuatInverse(CollisionR[i]);
					const FVector3f PhiWithNormalInputFloat = VectorQuaternionRotateVector(InverseR, TranslatedVec);
					FVector3f Normal;
					float Penetration;

					uniform int IMMask = 0;
					if(IM != 0)
					{
						IMMask = lanemask();
					}
			
					if(!GetSimplePenetrationWithNormal(PhiWithNormalInputFloat, Type, Ptr, TypedPtr, SizeofFImplicitObject, OffsetofGeometryType, OffsetOfMargin, CollisionThickness, Penetration, Normal))
					{
						const uniform int Mask = IMMask & lanemask();
						if(Mask != 0)  // Do not call c++ when there is no work to do
						{
							float Phi;
							GetPhiWithNormalCollisionParticleRange(CollisionParticlesRange, (const uniform float *uniform)&PhiWithNormalInputFloat, (uniform float *uniform)&Normal, (uniform float *uniform)&Phi, i, programCount, Mask);
							Penetration = CollisionThickness - Phi;
						}
					}

					if(Penetration > InnerThickness)
					{
						const uniform int Mask = IMMask & lanemask();
						if (Mask != 0)  // Do not call c++ when there is no work to do
						{
							const FVector3f X = VectorLoad(&ParticlesX[extract(Index, 0)]);
							ApplySimpleCCDNoFriction(
								CollisionParticlesRange,
								i,
								LastCollisionTransforms[i],
								(uniform float *uniform)&X,
								(uniform float* uniform)&CollisionThickness,
								(uniform float *uniform)&P,
								programCount,
								Mask);
						}
					}
					else if(Penetration > 0.0f)
					{
						// TransformVector. Scale is 1 so that part can be removed.
						const FVector3f NormalWorld = VectorQuaternionRotateVector(CollisionR[i], Normal);
						P = P + (Penetration * NormalWorld);
					}
				}

				if(IM != 0)
				{
					VectorStore(&ParticlesPandInvM[extract(Index, 0)], SetVector4(P, IM));
				}
			}
		}
		else  // No weight map
		{
			uniform float InnerThickness = InnerThicknessTable[0];

			foreach(Index = OuterParticleBatchBegin ... OuterParticleBatchEnd)
			{
				const FVector4f PandInvM = VectorLoad(&ParticlesPandInvM[extract(Index, 0)]);
				varying FVector3f P;
				varying float IM;
				UnzipPandInvM(PandInvM, P, IM);

				if(all(IM == 0))
				{
					continue;
				}
				
				varying float CollisionThickness = PerGroupCollisionThickness + SoftBodyThicknessOffsetRange.V[0];
				if (SoftBodyThicknessMap != NULL)
				{
					CollisionThickness += SoftBodyThicknessMap[Index] * SoftBodyThicknessOffsetRange.V[1];
				}

				for(uniform int i = 0; i < InnerCollisionCount; i++) // SequentialFor
				{
					const uniform SerializedPtr *uniform GeometryPtr = (const uniform SerializedPtr *uniform)Geometry;
					const uniform uint8 *uniform Ptr = GeometryPtr[i].Ptr;
					const uniform uint8 *uniform TypedPtr = Ptr + SizeofFImplicitObject;
					const uniform uint8 Type = Ptr[OffsetofGeometryType] & 0x3F;

					if(!IsSimpleCollider(Type))
					{
						continue;
					}

					// InverseTransformPosition. Can skip Scale calculation because it is 1 for xyz.
					const FVector3f TranslatedVec = P - CollisionX[i];
					const uniform FVector4f InverseR = QuatInverse(CollisionR[i]);
					const FVector3f PhiWithNormalInputFloat = VectorQuaternionRotateVector(InverseR, TranslatedVec);
					FVector3f Normal;
					float Penetration;

					uniform int IMMask = 0;
					if(IM != 0)
					{
						IMMask = lanemask();
					}
			
					if(!GetSimplePenetrationWithNormal(PhiWithNormalInputFloat, Type, Ptr, TypedPtr, SizeofFImplicitObject, OffsetofGeometryType, OffsetOfMargin, CollisionThickness, Penetration, Normal))
					{
						const uniform int Mask = IMMask & lanemask();
						if(Mask != 0)  // Do not call c++ when there is no work to do
						{
							float Phi;
							GetPhiWithNormalCollisionParticleRange(CollisionParticlesRange, (const uniform float *uniform)&PhiWithNormalInputFloat, (uniform float *uniform)&Normal, (uniform float *uniform)&Phi, i, programCount, Mask);
							Penetration = CollisionThickness - Phi;
						}
					}

					if(Penetration > InnerThickness)
					{
						const uniform int Mask = IMMask & lanemask();
						if (Mask != 0)  // Do not call c++ when there is no work to do
						{
							const FVector3f X = VectorLoad(&ParticlesX[extract(Index, 0)]);
							ApplySimpleCCDNoFriction(
								CollisionParticlesRange,
								i,
								LastCollisionTransforms[i],
								(uniform float *uniform)&X,
								(uniform float* uniform)&CollisionThickness,
								(uniform float *uniform)&P,
								programCount,
								Mask);
						}
					}
					else if(Penetration > 0.0f)
					{
						// TransformVector. Scale is 1 so that part can be removed.
						const FVector3f NormalWorld = VectorQuaternionRotateVector(CollisionR[i], Normal);
						P = P + (Penetration * NormalWorld);
					}
				}

				if(IM != 0)
				{
					VectorStore(&ParticlesPandInvM[extract(Index, 0)], SetVector4(P, IM));
				}
			}
		}
	}
	else  // No inner thickness
	{
		foreach(Index = OuterParticleBatchBegin ... OuterParticleBatchEnd)
		{
			const FVector4f PandInvM = VectorLoad(&ParticlesPandInvM[extract(Index, 0)]);
			varying FVector3f P;
			varying float IM;
			UnzipPandInvM(PandInvM, P, IM);

			if(all(IM == 0))
			{
				continue;
			}
			
			varying float CollisionThickness = PerGroupCollisionThickness + SoftBodyThicknessOffsetRange.V[0];
			if (SoftBodyThicknessMap != NULL)
			{
				CollisionThickness += SoftBodyThicknessMap[Index] * SoftBodyThicknessOffsetRange.V[1];
			}

			for(uniform int i = 0; i < InnerCollisionCount; i++) // SequentialFor
			{
				const uniform SerializedPtr *uniform GeometryPtr = (const uniform SerializedPtr *uniform)Geometry;
				const uniform uint8 *uniform Ptr = GeometryPtr[i].Ptr;
				const uniform uint8 *uniform TypedPtr = Ptr + SizeofFImplicitObject;
				const uniform uint8 Type = Ptr[OffsetofGeometryType] & 0x3F;

				if(!IsSimpleCollider(Type))
				{
					continue;
				}

				// InverseTransformPosition. Can skip Scale calculation because it is 1 for xyz.
				const FVector3f TranslatedVec = P - CollisionX[i];
				const uniform FVector4f InverseR = QuatInverse(CollisionR[i]);
				const FVector3f PhiWithNormalInputFloat = VectorQuaternionRotateVector(InverseR, TranslatedVec);
				FVector3f Normal;
				float Penetration;

			
				if(!GetSimplePenetrationWithNormal(PhiWithNormalInputFloat, Type, Ptr, TypedPtr, SizeofFImplicitObject, OffsetofGeometryType, OffsetOfMargin, CollisionThickness, Penetration, Normal))
				{
					uniform int IMMask = 0;
					if(IM != 0)
					{
						IMMask = lanemask();
					}
					if(IMMask != 0)  // Do not call c++ when there is no work to do
					{
						float Phi;
						GetPhiWithNormalCollisionParticleRange(CollisionParticlesRange, (const uniform float *uniform)&PhiWithNormalInputFloat, (uniform float *uniform)&Normal, (uniform float *uniform)&Phi, i, programCount, IMMask);
						Penetration = CollisionThickness - Phi;
					}
				}

				if(Penetration > 0.0f)
				{
					// TransformVector. Scale is 1 so that part can be removed.
					const FVector3f NormalWorld = VectorQuaternionRotateVector(CollisionR[i], Normal);
					P = P + (Penetration * NormalWorld);
				}
			}

			if(IM != 0)
			{
				VectorStore(&ParticlesPandInvM[extract(Index, 0)], SetVector4(P, IM));
			}
		}
	}
}

export void ApplyPerParticleSimpleCollisionNoFrictionAndGeneratePlanarConstraints(uniform FVector4f ParticlesPandInvM[],
												uniform bool HasPlanarData[],
												uniform FVector3f PlanarDataPositions[],
												uniform FVector3f PlanarDataNormals[],
												const uniform FVector3f ParticlesX[],
												const uniform FVector3f CollisionX[],
												const uniform FVector4f CollisionR[],
												const uniform float PerGroupCollisionThickness,
												const uniform FVector2f& SoftBodyThicknessOffsetRange,
												const uniform float SoftBodyThicknessMap[],
												const uniform uint8 *uniform CollisionParticlesRange,
												const uniform uint8 *uniform Geometry,
												const uniform int SizeofFImplicitObject,
												const uniform int OffsetofGeometryType,
												const uniform int OffsetOfMargin,
												const uniform int InnerCollisionCount,
												const uniform int OuterParticleBatchBegin,
												const uniform int OuterParticleBatchEnd)
{

	foreach(Index = OuterParticleBatchBegin ... OuterParticleBatchEnd)
	{
		const FVector4f PandInvM = VectorLoad(&ParticlesPandInvM[extract(Index, 0)]);
		varying FVector3f P;
		varying float IM;
		UnzipPandInvM(PandInvM, P, IM);

		varying bool bHasPlanarData = false;
		varying FVector3f PlanarP;
		varying FVector3f PlanarN;

		if(all(IM == 0))
		{
			continue;
		}
		
		varying float CollisionThickness = PerGroupCollisionThickness + SoftBodyThicknessOffsetRange.V[0];
		if (SoftBodyThicknessMap != NULL)
		{
			CollisionThickness += SoftBodyThicknessMap[Index] * SoftBodyThicknessOffsetRange.V[1];
		}

		for(uniform int i = 0; i < InnerCollisionCount; i++) // SequentialFor
		{
			const uniform SerializedPtr *uniform GeometryPtr = (const uniform SerializedPtr *uniform)Geometry;
			const uniform uint8 *uniform Ptr = GeometryPtr[i].Ptr;
			const uniform uint8 *uniform TypedPtr = Ptr + SizeofFImplicitObject;
			const uniform uint8 Type = Ptr[OffsetofGeometryType] & 0x3F;

			if(!IsSimpleCollider(Type))
			{
				continue;
			}

			// InverseTransformPosition. Can skip Scale calculation because it is 1 for xyz.
			const FVector3f TranslatedVec = P - CollisionX[i];
			const uniform FVector4f InverseR = QuatInverse(CollisionR[i]);
			const FVector3f PhiWithNormalInputFloat = VectorQuaternionRotateVector(InverseR, TranslatedVec);
			FVector3f Normal;
			float Penetration;
			
			if(!GetSimplePenetrationWithNormal(PhiWithNormalInputFloat, Type, Ptr, TypedPtr, SizeofFImplicitObject, OffsetofGeometryType, OffsetOfMargin, CollisionThickness, Penetration, Normal))
			{
				uniform int IMMask;
				if(IM != 0)
				{
					IMMask = lanemask();
				}
				
				float Phi;
				GetPhiWithNormalCollisionParticleRange(CollisionParticlesRange, (const uniform float *uniform)&PhiWithNormalInputFloat, (uniform float *uniform)&Normal, (uniform float *uniform)&Phi, i, programCount, IMMask);
				Penetration = CollisionThickness - Phi;
			}

			if(Penetration > 0.0f)
			{
				// TransformVector. Scale is 1 so that part can be removed.
				PlanarN = VectorQuaternionRotateVector(CollisionR[i], Normal);
				P = P + (Penetration * PlanarN);

				bHasPlanarData = true;
				PlanarP = P;
			}
		}

		if(IM != 0)
		{
			VectorStore(&ParticlesPandInvM[extract(Index, 0)], SetVector4(P, IM));

			if(bHasPlanarData)
			{
				HasPlanarData[Index] = true;
				VectorStore(&PlanarDataPositions[extract(Index,0)], PlanarP);
				VectorStore(&PlanarDataNormals[extract(Index,0)], PlanarN);
			}
		}
	}
}

