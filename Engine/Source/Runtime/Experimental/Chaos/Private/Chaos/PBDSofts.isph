// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef __PBDSOFTS_ISPH__
#define __PBDSOFTS_ISPH__
#define EXPLICIT_VECTOR4 1

#include "Math/Vector.isph"

inline void UnzipPandInvM( const FVector4f& PandInvM, FVector3f& P, float& InvM )
{
	P.V[0] = PandInvM.V[0];
	P.V[1] = PandInvM.V[1];
	P.V[2] = PandInvM.V[2];

	InvM = PandInvM.V[3];
}
#endif
