// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/PBDSphericalConstraint.h"
#include "Chaos/PBDParticles.h"
#include "Chaos/PBDRigidParticles.h"
#include "Chaos/Framework/Parallel.h"
#include "ChaosStats.h"
#include "ChaosLog.h"
#if INTEL_ISPC
#include "PBDSphericalConstraint.ispc.generated.h"
#endif

#if INTEL_ISPC
#if !UE_BUILD_SHIPPING || USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING
bool bChaos_Spherical_ISPC_Enabled = CHAOS_SPHERICAL_ISPC_ENABLED_DEFAULT;
FAutoConsoleVariableRef CVarChaosSphericalISPCEnabled(TEXT("p.Chaos.Spherical.ISPC"), bChaos_Spherical_ISPC_Enabled, TEXT("Whether to use ISPC optimizations in spherical constraints"));
#endif

static_assert(sizeof(ispc::FVector4f) == sizeof(Chaos::Softs::FPAndInvM), "sizeof(ispc::FVector4f) != sizeof(Chaos::Softs::FPAndInvM)");
#endif

namespace Chaos::Softs {

void FPBDSphericalConstraint::SetProperties(
	const FCollectionPropertyConstFacade& PropertyCollection,
	const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
	FSolverReal MeshScale)
{
	if (IsMaxDistanceMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedFloatMaxDistance(FVector2f::Max(GetWeightedFloatMaxDistance(PropertyCollection), FVector2f::ZeroVector));
		MaxDistanceBase = WeightedFloatMaxDistance[0];
		MaxDistanceRange = WeightedFloatMaxDistance[1] - MaxDistanceBase;

		if (IsMaxDistanceStringDirty(PropertyCollection))
		{
			SphereRadii = WeightMaps.FindRef(GetMaxDistanceString(PropertyCollection));
		}
	}

	SetScale(MeshScale);
}

template<typename SolverParticlesOrRange>
void FPBDSphericalConstraint::ApplyHelperISPC(SolverParticlesOrRange& Particles, const FSolverReal /*Dt*/) const
{
	check(bRealTypeCompatibleWithISPC);
	const TConstArrayView<FSolverVec3> AnimationPositionsView = Particles.GetConstArrayView(AnimationPositions);

#if INTEL_ISPC
	if (SphereRadii.Num() == ParticleCount)
	{
		ispc::ApplySphericalConstraintsWithMap(
			(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
			(const ispc::FVector3f*)AnimationPositionsView.GetData(),
			SphereRadii.GetData(),
			MaxDistanceBase * GetScale(),
			MaxDistanceRange * GetScale(),
			ParticleOffset,
			ParticleCount);
	}
	else
	{
		ispc::ApplySphericalConstraintsWithoutMap(
			(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
			(const ispc::FVector3f*)AnimationPositionsView.GetData(),
			MaxDistanceBase * GetScale(),
			ParticleOffset,
			ParticleCount);
	}
#endif
}
template CHAOS_API void FPBDSphericalConstraint::ApplyHelperISPC(FSolverParticles& Particles, const FSolverReal /*Dt*/) const;
template CHAOS_API void FPBDSphericalConstraint::ApplyHelperISPC(FSolverParticlesRange& Particles, const FSolverReal /*Dt*/) const;

void FPBDSphericalBackstopConstraint::SetProperties(
	const FCollectionPropertyConstFacade& PropertyCollection,
	const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
	FSolverReal MeshScale)
{
	if (IsBackstopRadiusMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedFloatBackstopRadius(FVector2f::Max(GetWeightedFloatBackstopRadius(PropertyCollection), FVector2f::ZeroVector));
		BackstopRadiusBase = (FSolverReal)WeightedFloatBackstopRadius[0];
		BackstopRadiusRange = (FSolverReal)WeightedFloatBackstopRadius[1] - BackstopRadiusBase;

		if (IsBackstopRadiusStringDirty(PropertyCollection))
		{
			SphereRadii = WeightMaps.FindRef(GetBackstopRadiusString(PropertyCollection));
		}

		// Update enable status using the radius property
		SetEnabled(IsBackstopRadiusEnabled(PropertyCollection));
	}

	if (IsBackstopDistanceMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedFloatBackstopDistance(GetWeightedFloatBackstopDistance(PropertyCollection));  // Unlike radius, distance can be negative
		BackstopDistanceBase = WeightedFloatBackstopDistance[0];
		BackstopDistanceRange = WeightedFloatBackstopDistance[1] - BackstopDistanceBase;

		if (IsBackstopDistanceStringDirty(PropertyCollection))
		{
			SphereOffsetDistances = WeightMaps.FindRef(GetBackstopDistanceString(PropertyCollection));
		}
	}

	SetScale(MeshScale);
}

void FPBDSphericalBackstopConstraint::ApplyLegacyHelperISPC(FSolverParticlesRange& Particles, const FSolverReal /*Dt*/) const
{
	check(bRealTypeCompatibleWithISPC);

	const TConstArrayView<FSolverVec3> AnimationPositionsView = bMeshUseGlobalIndexation ? Particles.GetConstArrayView(AnimationPositions) : TConstArrayView<FSolverVec3>(AnimationPositions);
	const TConstArrayView<FSolverVec3> AnimationNormalsView = bMeshUseGlobalIndexation ? Particles.GetConstArrayView(AnimationNormals) : TConstArrayView<FSolverVec3>(AnimationNormals);
#if INTEL_ISPC
	if (SphereRadii.Num() == ParticleCount)
	{
		if (SphereOffsetDistances.Num() == ParticleCount)
		{
			ispc::ApplyLegacySphericalBackstopConstraintsWithMaps(
				(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
				(const ispc::FVector3f*)AnimationPositionsView.GetData(),
				(const ispc::FVector3f*)AnimationNormalsView.GetData(),
				SphereOffsetDistances.GetData(),
				SphereRadii.GetData(),
				BackstopDistanceBase * GetScale(),
				BackstopDistanceRange * GetScale(),
				BackstopRadiusBase * GetScale(),
				BackstopRadiusRange * GetScale(),
				0,
				ParticleCount);
		}
		else
		{
			ispc::ApplyLegacySphericalBackstopConstraintsWithRadiusMap(
				(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
				(const ispc::FVector3f*)AnimationPositionsView.GetData(),
				(const ispc::FVector3f*)AnimationNormalsView.GetData(),
				SphereRadii.GetData(),
				BackstopDistanceBase * GetScale(),
				BackstopRadiusBase * GetScale(),
				BackstopRadiusRange * GetScale(),
				0,
				ParticleCount);
		}
	}
	else if (SphereOffsetDistances.Num() == ParticleCount)
	{
		ispc::ApplyLegacySphericalBackstopConstraintsWithDistanceMap(
			(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
			(const ispc::FVector3f*)AnimationPositionsView.GetData(),
			(const ispc::FVector3f*)AnimationNormalsView.GetData(),
			SphereOffsetDistances.GetData(),
			BackstopDistanceBase * GetScale(),
			BackstopDistanceRange * GetScale(),
			BackstopRadiusBase * GetScale(),
			0,
			ParticleCount);
	}
	else
	{
		ispc::ApplyLegacySphericalBackstopConstraintsWithoutMaps(
			(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
			(const ispc::FVector3f*)AnimationPositionsView.GetData(),
			(const ispc::FVector3f*)AnimationNormalsView.GetData(),
			BackstopDistanceBase * GetScale(),
			BackstopRadiusBase * GetScale(),
			0,
			ParticleCount);
	}
#endif
}

void FPBDSphericalBackstopConstraint::ApplyHelperISPC(FSolverParticlesRange& Particles, const FSolverReal /*Dt*/) const
{
	check(bRealTypeCompatibleWithISPC);

	const TConstArrayView<FSolverVec3> AnimationPositionsView = bMeshUseGlobalIndexation ? Particles.GetConstArrayView(AnimationPositions) : TConstArrayView<FSolverVec3>(AnimationPositions);
	const TConstArrayView<FSolverVec3> AnimationNormalsView = bMeshUseGlobalIndexation ? Particles.GetConstArrayView(AnimationNormals) : TConstArrayView<FSolverVec3>(AnimationNormals);
#if INTEL_ISPC
	if (SphereRadii.Num() == ParticleCount)
	{
		if (SphereOffsetDistances.Num() == ParticleCount)
		{
			ispc::ApplySphericalBackstopConstraintsWithMaps(
				(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
				(const ispc::FVector3f*)AnimationPositionsView.GetData(),
				(const ispc::FVector3f*)AnimationNormalsView.GetData(),
				SphereOffsetDistances.GetData(),
				SphereRadii.GetData(),
				BackstopDistanceBase * GetScale(),
				BackstopDistanceRange * GetScale(),
				BackstopRadiusBase * GetScale(),
				BackstopRadiusRange * GetScale(),
				0,
				ParticleCount);
		}
		else
		{
			ispc::ApplySphericalBackstopConstraintsWithRadiusMap(
				(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
				(const ispc::FVector3f*)AnimationPositionsView.GetData(),
				(const ispc::FVector3f*)AnimationNormalsView.GetData(),
				SphereRadii.GetData(),
				BackstopDistanceBase * GetScale(),
				BackstopRadiusBase * GetScale(),
				BackstopRadiusRange * GetScale(),
				0,
				ParticleCount);
		}
	}
	else if (SphereOffsetDistances.Num() == ParticleCount)
	{
		ispc::ApplySphericalBackstopConstraintsWithDistanceMap(
			(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
			(const ispc::FVector3f*)AnimationPositionsView.GetData(),
			(const ispc::FVector3f*)AnimationNormalsView.GetData(),
			SphereOffsetDistances.GetData(),
			BackstopDistanceBase * GetScale(),
			BackstopDistanceRange * GetScale(),
			BackstopRadiusBase * GetScale(),
			0,
			ParticleCount);
	}
	else
	{
		ispc::ApplySphericalBackstopConstraintsWithoutMaps(
			(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
			(const ispc::FVector3f*)AnimationPositionsView.GetData(),
			(const ispc::FVector3f*)AnimationNormalsView.GetData(),
			BackstopDistanceBase * GetScale(),
			BackstopRadiusBase * GetScale(),
			0,
			ParticleCount);
	}
#endif
}

}  // End namespace Chaos::Softs
