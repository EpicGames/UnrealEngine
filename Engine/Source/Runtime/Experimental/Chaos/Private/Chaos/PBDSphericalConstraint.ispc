// Copyright Epic Games, Inc. All Rights Reserved.

#define EXPLICIT_VECTOR4 1

#include "Math/Vector.isph"
#include "Chaos/PBDSofts.isph"

export void ApplySphericalConstraintsWithMap(uniform FVector4f ParticlesPandInvM[],
									const uniform FVector3f AnimationPositions[],
									const uniform float SphereRadii[],
									const uniform float SphereRadiiBase,
									const uniform float SphereRadiiRange,
									const uniform int32 ParticleOffset,
									const uniform int32 ParticleCount)
{
	static const float DeadZoneSquareRadius = FLOAT_SMALL_NUMBER; // We will not push the particle away in the dead zone

	foreach(Index = 0 ... ParticleCount)
	{
		const uniform int32 Offset = ParticleOffset + extract(Index, 0);
		const FVector4f PandInvM = VectorLoad(&ParticlesPandInvM[Offset]);

		varying FVector3f Position;
		varying float IM;
		UnzipPandInvM(PandInvM, Position, IM);

		if(all(IM == 0))
		{
			continue;
		}

		
		const FVector3f Center = VectorLoad(&AnimationPositions[Offset]);

		const FVector3f CenterToParticle = Position - Center;
		const float DistanceSquared = VectorSizeSquared(CenterToParticle);
		const float Distance = sqrt(DistanceSquared);

		const float Radius = SphereRadiiBase + SphereRadiiRange * SphereRadii[Index];
		const FVector3f FinalPosition = ((Radius / Distance) * CenterToParticle) + Center;

		if (IM != 0 && DistanceSquared > ((Radius * Radius) + DeadZoneSquareRadius))
		{
			VectorStore(&ParticlesPandInvM[Offset], SetVector4(FinalPosition, IM));
		}
	}
}

export void ApplySphericalConstraintsWithoutMap(uniform FVector4f ParticlesPandInvM[],
									const uniform FVector3f AnimationPositions[],
									const uniform float Radius,
									const uniform int32 ParticleOffset,
									const uniform int32 ParticleCount)
{
	static const float DeadZoneSquareRadius = FLOAT_SMALL_NUMBER; // We will not push the particle away in the dead zone

	foreach(Index = 0 ... ParticleCount)
	{
		const uniform int32 Offset = ParticleOffset + extract(Index, 0);
		const FVector4f PandInvM = VectorLoad(&ParticlesPandInvM[Offset]);

		varying FVector3f Position;
		varying float IM;
		UnzipPandInvM(PandInvM, Position, IM);

		if(all(IM == 0))
		{
			continue;
		}

		
		const FVector3f Center = VectorLoad(&AnimationPositions[Offset]);

		const FVector3f CenterToParticle = Position - Center;
		const float DistanceSquared = VectorSizeSquared(CenterToParticle);
		const float Distance = sqrt(DistanceSquared);

		const FVector3f FinalPosition = ((Radius / Distance) * CenterToParticle) + Center;

		if (IM != 0 && DistanceSquared > ((Radius * Radius) + DeadZoneSquareRadius))
		{
			VectorStore(&ParticlesPandInvM[Offset], SetVector4(FinalPosition, IM));
		}
	}
}

// Deprecated since 5.3
export void ApplySphericalConstraints(uniform FVector4f ParticlesPandInvM[],
									const uniform FVector3f AnimationPositions[],
									const uniform float SphereRadii[],
									const uniform float SphereRadiiMultiplier,
									const uniform int32 ParticleOffset,
									const uniform int32 ParticleCount)
{
	static const uniform float SphereRadiiBase = 0.f;

	ApplySphericalConstraintsWithMap(
		ParticlesPandInvM,
		AnimationPositions,
		SphereRadii,
		SphereRadiiBase,
		SphereRadiiMultiplier,
		ParticleOffset,
		ParticleCount);
}

export void ApplyLegacySphericalBackstopConstraintsWithMaps(uniform FVector4f ParticlesPandInvM[],
													const uniform FVector3f AnimationPositions[],
													const uniform FVector3f AnimationNormals[],
													const uniform float SphereOffsetDistances[],
													const uniform float SphereRadii[],
													const uniform float SphereOffsetDistancesBase,
													const uniform float SphereOffsetDistancesRange,
													const uniform float SphereRadiiBase,
													const uniform float SphereRadiiRange,
													const uniform int32 ParticleOffset,
													const uniform int32 ParticleCount)
{
	static const float DeadZoneSquareRadius = FLOAT_SMALL_NUMBER; // We will not push the particle away in the dead zone

	foreach(Index = 0 ... ParticleCount)
	{
		const uniform int32 Offset = ParticleOffset + extract(Index, 0);
		const FVector4f PandInvM = VectorLoad(&ParticlesPandInvM[Offset]);

		varying FVector3f Position;
		varying float IM;
		UnzipPandInvM(PandInvM, Position, IM);

		if(all(IM == 0))
		{
			continue;
		}

		const FVector3f AnimationPosition = VectorLoad(&AnimationPositions[Offset]);
		const FVector3f AnimationNormal = VectorLoad(&AnimationNormals[Offset]);

		const float SphereOffsetDistance = SphereOffsetDistancesBase + SphereOffsetDistancesRange * SphereOffsetDistances[Index];
		const float Radius = SphereRadiiBase + SphereRadiiRange * SphereRadii[Index];

		const FVector3f Center = AnimationPosition - SphereOffsetDistance * AnimationNormal;  // Legacy version already includes the radius within the distance
		const FVector3f CenterToParticle = Position - Center;
		const float DistanceSquared = VectorSizeSquared(CenterToParticle);

		const FVector3f DistSquaredLessRadiusSquared = ((Radius / sqrt(DistanceSquared)) * CenterToParticle) + Center;
		const FVector3f DistSquaredLessDeadZoneRadius = AnimationPosition - (SphereOffsetDistance - Radius) * AnimationNormal;  // Legacy version already includes the radius to the distance

		FVector3f NewP = VectorSelect(DistanceSquared < Radius * Radius, DistSquaredLessRadiusSquared, FloatZeroVector);
		NewP = VectorSelect(DistanceSquared < DeadZoneSquareRadius, DistSquaredLessDeadZoneRadius, NewP);

		if (IM != 0 && NewP.V[0] != 0)
		{
			VectorStore(&ParticlesPandInvM[Offset], SetVector4(NewP, IM));
		}
	}
}

export void ApplyLegacySphericalBackstopConstraintsWithDistanceMap(uniform FVector4f ParticlesPandInvM[],
													const uniform FVector3f AnimationPositions[],
													const uniform FVector3f AnimationNormals[],
													const uniform float SphereOffsetDistances[],
													const uniform float SphereOffsetDistancesBase,
													const uniform float SphereOffsetDistancesRange,
													const uniform float Radius,
													const uniform int32 ParticleOffset,
													const uniform int32 ParticleCount)
{
	static const float DeadZoneSquareRadius = FLOAT_SMALL_NUMBER; // We will not push the particle away in the dead zone

	foreach(Index = 0 ... ParticleCount)
	{
		const uniform int32 Offset = ParticleOffset + extract(Index, 0);
		const FVector4f PandInvM = VectorLoad(&ParticlesPandInvM[Offset]);

		varying FVector3f Position;
		varying float IM;
		UnzipPandInvM(PandInvM, Position, IM);

		if(all(IM == 0))
		{
			continue;
		}

		const FVector3f AnimationPosition = VectorLoad(&AnimationPositions[Offset]);
		const FVector3f AnimationNormal = VectorLoad(&AnimationNormals[Offset]);

		const float SphereOffsetDistance = SphereOffsetDistancesBase + SphereOffsetDistancesRange * SphereOffsetDistances[Index];

		const FVector3f Center = AnimationPosition - SphereOffsetDistance * AnimationNormal;  // Legacy version already includes the radius within the distance
		const FVector3f CenterToParticle = Position - Center;
		const float DistanceSquared = VectorSizeSquared(CenterToParticle);

		const FVector3f DistSquaredLessRadiusSquared = ((Radius / sqrt(DistanceSquared)) * CenterToParticle) + Center;
		const FVector3f DistSquaredLessDeadZoneRadius = AnimationPosition - (SphereOffsetDistance - Radius) * AnimationNormal;  // Legacy version already includes the radius to the distance

		FVector3f NewP = VectorSelect(DistanceSquared < Radius * Radius, DistSquaredLessRadiusSquared, FloatZeroVector);
		NewP = VectorSelect(DistanceSquared < DeadZoneSquareRadius, DistSquaredLessDeadZoneRadius, NewP);

		if (IM != 0 && NewP.V[0] != 0)
		{
			VectorStore(&ParticlesPandInvM[Offset], SetVector4(NewP, IM));
		}
	}
}

export void ApplyLegacySphericalBackstopConstraintsWithRadiusMap(uniform FVector4f ParticlesPandInvM[],
													const uniform FVector3f AnimationPositions[],
													const uniform FVector3f AnimationNormals[],
													const uniform float SphereRadii[],
													const uniform float SphereOffsetDistance,
													const uniform float SphereRadiiBase,
													const uniform float SphereRadiiRange,
													const uniform int32 ParticleOffset,
													const uniform int32 ParticleCount)
{
	static const float DeadZoneSquareRadius = FLOAT_SMALL_NUMBER; // We will not push the particle away in the dead zone

	foreach(Index = 0 ... ParticleCount)
	{
		const uniform int32 Offset = ParticleOffset + extract(Index, 0);
		const FVector4f PandInvM = VectorLoad(&ParticlesPandInvM[Offset]);

		varying FVector3f Position;
		varying float IM;
		UnzipPandInvM(PandInvM, Position, IM);

		if(all(IM == 0))
		{
			continue;
		}

		const FVector3f AnimationPosition = VectorLoad(&AnimationPositions[Offset]);
		const FVector3f AnimationNormal = VectorLoad(&AnimationNormals[Offset]);

		const float Radius = SphereRadiiBase + SphereRadiiRange * SphereRadii[Index];

		const FVector3f Center = AnimationPosition - SphereOffsetDistance * AnimationNormal;  // Legacy version already includes the radius within the distance
		const FVector3f CenterToParticle = Position - Center;
		const float DistanceSquared = VectorSizeSquared(CenterToParticle);

		const FVector3f DistSquaredLessRadiusSquared = ((Radius / sqrt(DistanceSquared)) * CenterToParticle) + Center;
		const FVector3f DistSquaredLessDeadZoneRadius = AnimationPosition - (SphereOffsetDistance - Radius) * AnimationNormal;  // Legacy version already includes the radius to the distance

		FVector3f NewP = VectorSelect(DistanceSquared < Radius * Radius, DistSquaredLessRadiusSquared, FloatZeroVector);
		NewP = VectorSelect(DistanceSquared < DeadZoneSquareRadius, DistSquaredLessDeadZoneRadius, NewP);

		if (IM != 0 && NewP.V[0] != 0)
		{
			VectorStore(&ParticlesPandInvM[Offset], SetVector4(NewP, IM));
		}
	}
}

export void ApplyLegacySphericalBackstopConstraintsWithoutMaps(uniform FVector4f ParticlesPandInvM[],
													const uniform FVector3f AnimationPositions[],
													const uniform FVector3f AnimationNormals[],
													const uniform float SphereOffsetDistance,
													const uniform float Radius,
													const uniform int32 ParticleOffset,
													const uniform int32 ParticleCount)
{
	static const float DeadZoneSquareRadius = FLOAT_SMALL_NUMBER; // We will not push the particle away in the dead zone

	foreach(Index = 0 ... ParticleCount)
	{
		const uniform int32 Offset = ParticleOffset + extract(Index, 0);
		const FVector4f PandInvM = VectorLoad(&ParticlesPandInvM[Offset]);

		varying FVector3f Position;
		varying float IM;
		UnzipPandInvM(PandInvM, Position, IM);

		if(all(IM == 0))
		{
			continue;
		}

		const FVector3f AnimationPosition = VectorLoad(&AnimationPositions[Offset]);
		const FVector3f AnimationNormal = VectorLoad(&AnimationNormals[Offset]);

		const FVector3f Center = AnimationPosition - SphereOffsetDistance * AnimationNormal;  // Legacy version already includes the radius within the distance
		const FVector3f CenterToParticle = Position - Center;
		const float DistanceSquared = VectorSizeSquared(CenterToParticle);

		const FVector3f DistSquaredLessRadiusSquared = ((Radius / sqrt(DistanceSquared)) * CenterToParticle) + Center;
		const FVector3f DistSquaredLessDeadZoneRadius = AnimationPosition - (SphereOffsetDistance - Radius) * AnimationNormal;  // Legacy version already includes the radius to the distance

		FVector3f NewP = VectorSelect(DistanceSquared < Radius * Radius, DistSquaredLessRadiusSquared, FloatZeroVector);
		NewP = VectorSelect(DistanceSquared < DeadZoneSquareRadius, DistSquaredLessDeadZoneRadius, NewP);

		if (IM != 0 && NewP.V[0] != 0)
		{
			VectorStore(&ParticlesPandInvM[Offset], SetVector4(NewP, IM));
		}
	}
}

// Deprecated since 5.3
export void ApplyLegacySphericalBackstopConstraints(uniform FVector4f ParticlesPandInvM[],
													const uniform FVector3f AnimationPositions[],
													const uniform FVector3f AnimationNormals[],
													const uniform float SphereOffsetDistances[],
													const uniform float SphereRadii[],
													const uniform float SphereRadiiMultiplier,
													const uniform int32 ParticleOffset,
													const uniform int32 ParticleCount)
{
	static const uniform float SphereOffsetDistancesBase = 0.f;
	static const uniform float SphereOffsetDistancesRange = 1.f;
	static const uniform float SphereRadiiBase = 0.f;

	ApplyLegacySphericalBackstopConstraintsWithMaps(
		ParticlesPandInvM,
		AnimationPositions,
		AnimationNormals,
		SphereOffsetDistances,
		SphereRadii,
		SphereOffsetDistancesBase,
		SphereOffsetDistancesRange,
		SphereRadiiBase,
		SphereRadiiMultiplier,
		ParticleOffset,
		ParticleCount);
}

export void ApplySphericalBackstopConstraintsWithMaps(uniform FVector4f ParticlesPandInvM[],
												const uniform FVector3f AnimationPositions[],
												const uniform FVector3f AnimationNormals[],
												const uniform float SphereOffsetDistances[],
												const uniform float SphereRadii[],
												const uniform float SphereOffsetDistancesBase,
												const uniform float SphereOffsetDistancesRange,
												const uniform float SphereRadiiBase,
												const uniform float SphereRadiiRange,
												const uniform int32 ParticleOffset,
												const uniform int32 ParticleCount)
{
	static const float DeadZoneSquareRadius = FLOAT_SMALL_NUMBER; // We will not push the particle away in the dead zone

	foreach(Index = 0 ... ParticleCount)
	{
		const uniform int32 Offset = ParticleOffset + extract(Index, 0);
		const FVector4f PandInvM = VectorLoad(&ParticlesPandInvM[Offset]);

		varying FVector3f Position;
		varying float IM;
		UnzipPandInvM(PandInvM, Position, IM);

		if(all(IM == 0))
		{
			continue;
		}

		const FVector3f AnimationPosition = VectorLoad(&AnimationPositions[Offset]);
		const FVector3f AnimationNormal = VectorLoad(&AnimationNormals[Offset]);

		const float SphereOffsetDistance = SphereOffsetDistancesBase + SphereOffsetDistancesRange * SphereOffsetDistances[Index];
		const float Radius = SphereRadiiBase + SphereRadiiRange * SphereRadii[Index];

		const FVector3f Center = AnimationPosition - (Radius + SphereOffsetDistance) * AnimationNormal;  // Non legacy version adds radius to the distance
		const FVector3f CenterToParticle = Position - Center;
		const float DistanceSquared = VectorSizeSquared(CenterToParticle);

		const FVector3f DistSquaredLessRadiusSquared = ((Radius / sqrt(DistanceSquared)) * CenterToParticle) + Center;
		const FVector3f DistSquaredLessDeadZoneRadius = AnimationPosition - SphereOffsetDistance * AnimationNormal;  // Non legacy version adds radius to the distance

		FVector3f NewP = VectorSelect(DistanceSquared < Radius * Radius, DistSquaredLessRadiusSquared, FloatZeroVector);
		NewP = VectorSelect(DistanceSquared < DeadZoneSquareRadius, DistSquaredLessDeadZoneRadius, NewP);

		if (IM != 0 && NewP.V[0] != 0)
		{
			VectorStore(&ParticlesPandInvM[Offset], SetVector4(NewP, IM));
		}
	}
}

export void ApplySphericalBackstopConstraintsWithDistanceMap(uniform FVector4f ParticlesPandInvM[],
												const uniform FVector3f AnimationPositions[],
												const uniform FVector3f AnimationNormals[],
												const uniform float SphereOffsetDistances[],
												const uniform float SphereOffsetDistancesBase,
												const uniform float SphereOffsetDistancesRange,
												const uniform float Radius,
												const uniform int32 ParticleOffset,
												const uniform int32 ParticleCount)
{
	static const float DeadZoneSquareRadius = FLOAT_SMALL_NUMBER; // We will not push the particle away in the dead zone

	foreach(Index = 0 ... ParticleCount)
	{
		const uniform int32 Offset = ParticleOffset + extract(Index, 0);
		const FVector4f PandInvM = VectorLoad(&ParticlesPandInvM[Offset]);

		varying FVector3f Position;
		varying float IM;
		UnzipPandInvM(PandInvM, Position, IM);

		if(all(IM == 0))
		{
			continue;
		}

		const FVector3f AnimationPosition = VectorLoad(&AnimationPositions[Offset]);
		const FVector3f AnimationNormal = VectorLoad(&AnimationNormals[Offset]);

		const float SphereOffsetDistance = SphereOffsetDistancesBase + SphereOffsetDistancesRange * SphereOffsetDistances[Index];

		const FVector3f Center = AnimationPosition - (Radius + SphereOffsetDistance) * AnimationNormal;  // Non legacy version adds radius to the distance
		const FVector3f CenterToParticle = Position - Center;
		const float DistanceSquared = VectorSizeSquared(CenterToParticle);

		const FVector3f DistSquaredLessRadiusSquared = ((Radius / sqrt(DistanceSquared)) * CenterToParticle) + Center;
		const FVector3f DistSquaredLessDeadZoneRadius = AnimationPosition - SphereOffsetDistance * AnimationNormal;  // Non legacy version adds radius to the distance

		FVector3f NewP = VectorSelect(DistanceSquared < Radius * Radius, DistSquaredLessRadiusSquared, FloatZeroVector);
		NewP = VectorSelect(DistanceSquared < DeadZoneSquareRadius, DistSquaredLessDeadZoneRadius, NewP);

		if (IM != 0 && NewP.V[0] != 0)
		{
			VectorStore(&ParticlesPandInvM[Offset], SetVector4(NewP, IM));
		}
	}
}

export void ApplySphericalBackstopConstraintsWithRadiusMap(uniform FVector4f ParticlesPandInvM[],
												const uniform FVector3f AnimationPositions[],
												const uniform FVector3f AnimationNormals[],
												const uniform float SphereRadii[],
												const uniform float SphereOffsetDistance,
												const uniform float SphereRadiiBase,
												const uniform float SphereRadiiRange,
												const uniform int32 ParticleOffset,
												const uniform int32 ParticleCount)
{
	static const float DeadZoneSquareRadius = FLOAT_SMALL_NUMBER; // We will not push the particle away in the dead zone

	foreach(Index = 0 ... ParticleCount)
	{
		const uniform int32 Offset = ParticleOffset + extract(Index, 0);
		const FVector4f PandInvM = VectorLoad(&ParticlesPandInvM[Offset]);

		varying FVector3f Position;
		varying float IM;
		UnzipPandInvM(PandInvM, Position, IM);

		if(all(IM == 0))
		{
			continue;
		}

		const FVector3f AnimationPosition = VectorLoad(&AnimationPositions[Offset]);
		const FVector3f AnimationNormal = VectorLoad(&AnimationNormals[Offset]);

		const float Radius = SphereRadiiBase + SphereRadiiRange * SphereRadii[Index];

		const FVector3f Center = AnimationPosition - (Radius + SphereOffsetDistance) * AnimationNormal;  // Non legacy version adds radius to the distance
		const FVector3f CenterToParticle = Position - Center;
		const float DistanceSquared = VectorSizeSquared(CenterToParticle);

		const FVector3f DistSquaredLessRadiusSquared = ((Radius / sqrt(DistanceSquared)) * CenterToParticle) + Center;
		const FVector3f DistSquaredLessDeadZoneRadius = AnimationPosition - SphereOffsetDistance * AnimationNormal;  // Non legacy version adds radius to the distance

		FVector3f NewP = VectorSelect(DistanceSquared < Radius * Radius, DistSquaredLessRadiusSquared, FloatZeroVector);
		NewP = VectorSelect(DistanceSquared < DeadZoneSquareRadius, DistSquaredLessDeadZoneRadius, NewP);

		if (IM != 0 && NewP.V[0] != 0)
		{
			VectorStore(&ParticlesPandInvM[Offset], SetVector4(NewP, IM));
		}
	}
}

export void ApplySphericalBackstopConstraintsWithoutMaps(uniform FVector4f ParticlesPandInvM[],
												const uniform FVector3f AnimationPositions[],
												const uniform FVector3f AnimationNormals[],
												const uniform float SphereOffsetDistance,
												const uniform float Radius,
												const uniform int32 ParticleOffset,
												const uniform int32 ParticleCount)
{
	static const float DeadZoneSquareRadius = FLOAT_SMALL_NUMBER; // We will not push the particle away in the dead zone

	foreach(Index = 0 ... ParticleCount)
	{
		const uniform int32 Offset = ParticleOffset + extract(Index, 0);
		const FVector4f PandInvM = VectorLoad(&ParticlesPandInvM[Offset]);

		varying FVector3f Position;
		varying float IM;
		UnzipPandInvM(PandInvM, Position, IM);

		if(all(IM == 0))
		{
			continue;
		}

		const FVector3f AnimationPosition = VectorLoad(&AnimationPositions[Offset]);
		const FVector3f AnimationNormal = VectorLoad(&AnimationNormals[Offset]);

		const FVector3f Center = AnimationPosition - (Radius + SphereOffsetDistance) * AnimationNormal;  // Non legacy version adds radius to the distance
		const FVector3f CenterToParticle = Position - Center;
		const float DistanceSquared = VectorSizeSquared(CenterToParticle);

		const FVector3f DistSquaredLessRadiusSquared = ((Radius / sqrt(DistanceSquared)) * CenterToParticle) + Center;
		const FVector3f DistSquaredLessDeadZoneRadius = AnimationPosition - SphereOffsetDistance * AnimationNormal;  // Non legacy version adds radius to the distance

		FVector3f NewP = VectorSelect(DistanceSquared < Radius * Radius, DistSquaredLessRadiusSquared, FloatZeroVector);
		NewP = VectorSelect(DistanceSquared < DeadZoneSquareRadius, DistSquaredLessDeadZoneRadius, NewP);

		if (IM != 0 && NewP.V[0] != 0)
		{
			VectorStore(&ParticlesPandInvM[Offset], SetVector4(NewP, IM));
		}
	}
}

// Deprecated since 5.3
export void ApplySphericalBackstopConstraints(uniform FVector4f ParticlesPandInvM[],
												const uniform FVector3f AnimationPositions[],
												const uniform FVector3f AnimationNormals[],
												const uniform float SphereOffsetDistances[],
												const uniform float SphereRadii[],
												const uniform float SphereRadiiMultiplier,
												const uniform int32 ParticleOffset,
												const uniform int32 ParticleCount)
{
	static const uniform float SphereOffsetDistancesBase = 0.f;
	static const uniform float SphereOffsetDistancesRange = 1.f;
	static const uniform float SphereRadiiBase = 0.f;

	ApplySphericalBackstopConstraintsWithMaps(
		ParticlesPandInvM,
		AnimationPositions,
		AnimationNormals,
		SphereOffsetDistances,
		SphereRadii,
		SphereOffsetDistancesBase,
		SphereOffsetDistancesRange,
		SphereRadiiBase,
		SphereRadiiMultiplier,
		ParticleOffset,
		ParticleCount);
}
