// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/PBDSpringConstraints.h"
#include "Chaos/PBDSoftsSolverParticles.h"
#include "Chaos/PBDRigidParticles.h"
#include "Chaos/GraphColoring.h"
#include "Chaos/Framework/Parallel.h"
#include "Chaos/TriangleMesh.h"
#include "ChaosStats.h"
#include "ChaosLog.h"
#include "HAL/IConsoleManager.h"

#if INTEL_ISPC
#include "PBDSpringConstraints.ispc.generated.h"
#endif

DECLARE_CYCLE_STAT(TEXT("Chaos PBD Spring Constraint"), STAT_PBD_Spring, STATGROUP_Chaos);

#if INTEL_ISPC
static_assert(sizeof(ispc::FVector4f) == sizeof(Chaos::Softs::FPAndInvM), "sizeof(ispc::FVector4f) != sizeof(Chaos::Softs::FPAndInvM)");
static_assert(sizeof(ispc::FIntVector2) == sizeof(Chaos::TVec2<int32>), "sizeof(ispc::FIntVector2) != sizeof(Chaos::TVec2<int32>)");

#if !UE_BUILD_SHIPPING || USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING
bool bChaos_Spring_ISPC_Enabled = CHAOS_SPRING_ISPC_ENABLED_DEFAULT;
FAutoConsoleVariableRef CVarChaosSpringISPCEnabled(TEXT("p.Chaos.Spring.ISPC"), bChaos_Spring_ISPC_Enabled, TEXT("Whether to use ISPC optimizations in Spring constraints"));
#endif
#endif

namespace Chaos::Softs {

// @todo(chaos): the parallel threshold (or decision to run parallel) should probably be owned by the solver and passed to the constraint container
static int32 Chaos_Spring_ParallelConstraintCount = 100;
#if !UE_BUILD_SHIPPING
FAutoConsoleVariableRef CVarChaosSpringParallelConstraintCount(TEXT("p.Chaos.Spring.ParallelConstraintCount"), Chaos_Spring_ParallelConstraintCount, TEXT("If we have more constraints than this, use parallel-for in Apply."));
#endif

template<typename SolverParticlesOrRange>
TArray<int32> FPBDSpringConstraints::InitColor(const SolverParticlesOrRange& Particles)
{
	TArray<int32> OrigToReorderedIndices; // used to reorder stiffness indices
	
	// In dev builds we always color so we can tune the system without restarting. See Apply()
#if UE_BUILD_SHIPPING || UE_BUILD_TEST
	if (Constraints.Num() > Chaos_Spring_ParallelConstraintCount)
#endif
	{
		const TArray<TArray<int32>> ConstraintsPerColor = FGraphColoring::ComputeGraphColoringParticlesOrRange(Constraints, Particles, ParticleOffset, ParticleOffset + ParticleCount);
		
		// Reorder constraints based on color so each array in ConstraintsPerColor contains contiguous elements.
		TArray<TVec2<int32>> ReorderedConstraints;
		TArray<FSolverReal> ReorderedDists;
		ReorderedConstraints.SetNumUninitialized(Constraints.Num());
		ReorderedDists.SetNumUninitialized(Dists.Num());
		OrigToReorderedIndices.SetNumUninitialized(Constraints.Num());

		ConstraintsPerColorStartIndex.Reset(ConstraintsPerColor.Num() + 1);

		int32 ReorderedIndex = 0;
		for (const TArray<int32>& ConstraintsBatch : ConstraintsPerColor)
		{
			ConstraintsPerColorStartIndex.Add(ReorderedIndex);
			for (const int32& BatchConstraint : ConstraintsBatch)
			{
				const int32 OrigIndex = BatchConstraint;
				ReorderedConstraints[ReorderedIndex] = Constraints[OrigIndex];
				ReorderedDists[ReorderedIndex] = Dists[OrigIndex];
				OrigToReorderedIndices[OrigIndex] = ReorderedIndex;

				++ReorderedIndex;
			}
		}
		ConstraintsPerColorStartIndex.Add(ReorderedIndex);

		Constraints = MoveTemp(ReorderedConstraints);
		Dists = MoveTemp(ReorderedDists);
		Stiffness.ReorderIndices(OrigToReorderedIndices);
	}
	return OrigToReorderedIndices;
}
template CHAOS_API TArray<int32> FPBDSpringConstraints::InitColor(const FSolverParticles& Particles);
template CHAOS_API TArray<int32> FPBDSpringConstraints::InitColor(const FSolverParticlesRange& Particles);

template<typename SolverParticlesOrRange>
void FPBDSpringConstraints::ApplyHelper(SolverParticlesOrRange& Particles, const FSolverReal Dt, const int32 ConstraintIndex, const FSolverReal ExpStiffnessValue) const
{
	const TVec2<int32>& Constraint = Constraints[ConstraintIndex];
	const int32 i1 = Constraint[0];
	const int32 i2 = Constraint[1];
	const FSolverVec3 Delta =  Base::GetDelta(Particles, ConstraintIndex, ExpStiffnessValue);
	if (Particles.InvM(i1) > (FSolverReal)0.)
	{
		Particles.P(i1) -= Particles.InvM(i1) * Delta;
	}
	if (Particles.InvM(i2) > (FSolverReal)0.)
	{
		Particles.P(i2) += Particles.InvM(i2) * Delta;
	}
}

template<typename SolverParticlesOrRange>
void FPBDSpringConstraints::Apply(SolverParticlesOrRange& Particles, const FSolverReal Dt) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPBDSpringConstraints_Apply);
	SCOPE_CYCLE_COUNTER(STAT_PBD_Spring);
	if ((ConstraintsPerColorStartIndex.Num() > 1) && (Constraints.Num() > Chaos_Spring_ParallelConstraintCount))
	{
		const int32 ConstraintColorNum = ConstraintsPerColorStartIndex.Num() - 1;
		if (!Stiffness.HasWeightMap())
		{
			const FSolverReal ExpStiffnessValue = (FSolverReal)Stiffness;

#if INTEL_ISPC
			if (bRealTypeCompatibleWithISPC && bChaos_Spring_ISPC_Enabled)
			{
				for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
				{
					const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
					const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
					ispc::ApplySpringConstraints(
						(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
						(ispc::FIntVector2*)&Constraints.GetData()[ColorStart],
						&Dists.GetData()[ColorStart],
						ExpStiffnessValue,
						ColorSize);
				}
			}
			else
#endif
			{
				for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
				{
					const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
					const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
					PhysicsParallelFor(ColorSize, [this, &Particles, Dt, ColorStart, ExpStiffnessValue](const int32 Index)
					{
						const int32 ConstraintIndex = ColorStart + Index;
						ApplyHelper(Particles, Dt, ConstraintIndex, ExpStiffnessValue);
					});
				}
			}
		}
		else  // Has weight maps
		{
#if INTEL_ISPC
			if (bRealTypeCompatibleWithISPC && bChaos_Spring_ISPC_Enabled)
			{
				for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
				{
					const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
					const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
					ispc::ApplySpringConstraintsWithWeightMaps(
						(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
						(ispc::FIntVector2*) & Constraints.GetData()[ColorStart],
						&Dists.GetData()[ColorStart],
						&Stiffness.GetIndices().GetData()[ColorStart],
						&Stiffness.GetTable().GetData()[0],
						ColorSize);
				}
			}
			else
#endif
			{
				for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
				{
					const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
					const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
					PhysicsParallelFor(ColorSize, [this, &Particles, Dt, ColorStart](const int32 Index)
					{
						const int32 ConstraintIndex = ColorStart + Index;
						const FSolverReal ExpStiffnessValue = Stiffness[ConstraintIndex];
						ApplyHelper(Particles, Dt, ConstraintIndex, ExpStiffnessValue);
					});
				}
			}
		}
	}
	else
	{
		if (!Stiffness.HasWeightMap())
		{
			const FSolverReal ExpStiffnessValue = (FSolverReal)Stiffness;
			for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
			{
				ApplyHelper(Particles, Dt, ConstraintIndex, ExpStiffnessValue);
			}
		}
		else
		{
			for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
			{
				const FSolverReal ExpStiffnessValue = Stiffness[ConstraintIndex];
				ApplyHelper(Particles, Dt, ConstraintIndex, ExpStiffnessValue);
			}
		}
	}
}
template CHAOS_API void FPBDSpringConstraints::Apply(FSolverParticles& Particles, const FSolverReal Dt) const;
template CHAOS_API void FPBDSpringConstraints::Apply(FSolverParticlesRange& Particles, const FSolverReal Dt) const;

FPBDEdgeSpringConstraints::FPBDEdgeSpringConstraints(
	const FSolverParticlesRange& Particles,
	const FTriangleMesh& TriangleMesh,
	const TArray<TVec3<FVec2f>>& FaceVertexPatternPositions,
	const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
	const FCollectionPropertyConstFacade& PropertyCollection,
	bool bTrimKinematicConstraints)
	: FPBDSpringConstraints(
		Particles,
		TriangleMesh.GetElements(),
		WeightMaps.FindRef(GetEdgeSpringStiffnessString(PropertyCollection, EdgeSpringStiffnessName.ToString())),
		FSolverVec2(GetWeightedFloatEdgeSpringStiffness(PropertyCollection, 1.f)),
		bTrimKinematicConstraints,
		false /*bInitColor*/)
	, WarpScale(FSolverVec2(GetWeightedFloatEdgeSpringWarpScale(PropertyCollection, 1.f)),
		WeightMaps.FindRef(GetEdgeSpringWarpScaleString(PropertyCollection, EdgeSpringWarpScaleName.ToString())),
		TConstArrayView<TVec2<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, WeftScale(FSolverVec2(GetWeightedFloatEdgeSpringWeftScale(PropertyCollection, 1.f)),
		WeightMaps.FindRef(GetEdgeSpringWeftScaleString(PropertyCollection, EdgeSpringWeftScaleName.ToString())),
		TConstArrayView<TVec2<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, EdgeSpringStiffnessIndex(PropertyCollection)
	, EdgeSpringWarpScaleIndex(PropertyCollection)
	, EdgeSpringWeftScaleIndex(PropertyCollection)
{
	if (EdgeSpringWarpScaleIndex != INDEX_NONE || EdgeSpringWeftScaleIndex != INDEX_NONE)
	{
		InitFromPatternData(FaceVertexPatternPositions, TriangleMesh);
	}

	const TArray<int32> OrigToReorderedIndices = InitColor(Particles);
	if (bWarpWeftScaleEnabled && OrigToReorderedIndices.Num() == Constraints.Num())
	{
		TArray<FSolverReal> ReorderedBaseDists;
		TArray<FSolverVec2> ReorderedWarpWeftScaleBaseMultipliers;
		ReorderedBaseDists.SetNumUninitialized(Constraints.Num());
		ReorderedWarpWeftScaleBaseMultipliers.SetNumUninitialized(Constraints.Num());
		for (int32 OrigIndex = 0; OrigIndex < Constraints.Num(); ++OrigIndex)
		{
			const int32 ReorderedIndex = OrigToReorderedIndices[OrigIndex];
			ReorderedBaseDists[ReorderedIndex] = BaseDists[OrigIndex];
			ReorderedWarpWeftScaleBaseMultipliers[ReorderedIndex] = WarpWeftScaleBaseMultipliers[OrigIndex];
		}

		BaseDists = MoveTemp(ReorderedBaseDists);
		WarpWeftScaleBaseMultipliers = MoveTemp(ReorderedWarpWeftScaleBaseMultipliers);
		WarpScale.ReorderIndices(OrigToReorderedIndices);
		WeftScale.ReorderIndices(OrigToReorderedIndices);
	}
}

void FPBDEdgeSpringConstraints::InitFromPatternData(const TArray<TVec3<FVec2f>>& FaceVertexPatternPositions, const FTriangleMesh& TriangleMesh)
{
	// Calculate multipliers per face. Average the multipliers for faces that share the same edge.
	TMap<TVec2<int32> /*Edge*/, TArray<FSolverVec2>> EdgeBasedWarpWeftScaleBaseMultiplier;
	auto SortedEdge = [](int32 P0, int32 P1) { return P0 <= P1 ? TVec2<int32>(P0, P1) : TVec2<int32>(P1, P0); };
	auto Multiplier = [](const FVec2f& UV0, const FVec2f& UV1)
		{
			const FSolverVec2 UVDiff = UV1 - UV0;
			const FSolverVec2 UVDiffAbs = UVDiff.GetAbs();
			const FSolverReal UVLength = UVDiffAbs.Length();
			return UVLength > UE_SMALL_NUMBER ? UVDiffAbs / UVLength : FSolverVec2(UE_INV_SQRT_2, UE_INV_SQRT_2); // Default to equally scaling warp and weft directions if zero length
		};

	const TArray<TVec3<int32>>& Elements = TriangleMesh.GetElements();
	for (int32 ElemIdx = 0; ElemIdx < Elements.Num(); ++ElemIdx)
	{
		const TVec3<int32>& Element = Elements[ElemIdx];
		const TVec3<FVec2f>& UVs = FaceVertexPatternPositions[ElemIdx];
		EdgeBasedWarpWeftScaleBaseMultiplier.FindOrAdd(SortedEdge(Element[0], Element[1])).Add(Multiplier(UVs[0], UVs[1]));
		EdgeBasedWarpWeftScaleBaseMultiplier.FindOrAdd(SortedEdge(Element[1], Element[2])).Add(Multiplier(UVs[1], UVs[2]));
		EdgeBasedWarpWeftScaleBaseMultiplier.FindOrAdd(SortedEdge(Element[2], Element[0])).Add(Multiplier(UVs[2], UVs[0]));
	}

	BaseDists = Dists;

	WarpWeftScaleBaseMultipliers.SetNumZeroed(Constraints.Num());
	for (int32 ConstraintIdx = 0; ConstraintIdx < Constraints.Num(); ++ConstraintIdx)
	{
		const TArray<FSolverVec2>& EdgeMultipliers = EdgeBasedWarpWeftScaleBaseMultiplier.FindChecked(SortedEdge(Constraints[ConstraintIdx][0], Constraints[ConstraintIdx][1]));
		check(EdgeMultipliers.Num());
		for (const FSolverVec2& EdgeMult : EdgeMultipliers)
		{
			WarpWeftScaleBaseMultipliers[ConstraintIdx] += EdgeMult;
		}
		WarpWeftScaleBaseMultipliers[ConstraintIdx].Normalize();
	}

	bWarpWeftScaleEnabled = true;
}

void FPBDEdgeSpringConstraints::SetProperties(
	const FCollectionPropertyConstFacade& PropertyCollection,
	const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps)
{
	if (IsEdgeSpringStiffnessMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedValue(GetWeightedFloatEdgeSpringStiffness(PropertyCollection));
		if (IsEdgeSpringStiffnessStringDirty(PropertyCollection))
		{
			const FString& WeightMapName = GetEdgeSpringStiffnessString(PropertyCollection);
			Stiffness = FPBDStiffness(
				WeightedValue,
				WeightMaps.FindRef(WeightMapName),
				TConstArrayView<TVec2<int32>>(Constraints),
				ParticleOffset,
				ParticleCount);
		}
		else
		{
			Stiffness.SetWeightedValue(WeightedValue);
		}
	}
	if (bWarpWeftScaleEnabled)
	{
		if (IsEdgeSpringWarpScaleMutable(PropertyCollection))
		{
			const FSolverVec2 WeightedValue(GetWeightedFloatEdgeSpringWarpScale(PropertyCollection));
			if (IsEdgeSpringWarpScaleStringDirty(PropertyCollection))
			{
				const FString& WeightMapName = GetEdgeSpringWarpScaleString(PropertyCollection);
				WarpScale = FPBDWeightMap(
					WeightedValue,
					WeightMaps.FindRef(WeightMapName),
					TConstArrayView<TVec2<int32>>(Constraints),
					ParticleOffset,
					ParticleCount);
			}
			else
			{
				WarpScale.SetWeightedValue(WeightedValue);
			}
		}
		if (IsEdgeSpringWeftScaleMutable(PropertyCollection))
		{
			const FSolverVec2 WeightedValue(GetWeightedFloatEdgeSpringWeftScale(PropertyCollection));
			if (IsEdgeSpringWeftScaleStringDirty(PropertyCollection))
			{
				const FString& WeightMapName = GetEdgeSpringWeftScaleString(PropertyCollection);
				WeftScale = FPBDWeightMap(
					WeightedValue,
					WeightMaps.FindRef(WeightMapName),
					TConstArrayView<TVec2<int32>>(Constraints),
					ParticleOffset,
					ParticleCount);
			}
			else
			{
				WeftScale.SetWeightedValue(WeightedValue);
			}
		}
	}
}

void FPBDEdgeSpringConstraints::ApplyProperties(const FSolverReal Dt, const int32 NumIterations)
{
	FPBDSpringConstraints::ApplyProperties(Dt, NumIterations);

	if (bWarpWeftScaleEnabled)
	{
		bool bWarpScaleChanged = false;
		WarpScale.ApplyValues(&bWarpScaleChanged);
		bool bWeftScaleChanged = false;
		WeftScale.ApplyValues(&bWeftScaleChanged);
		if (bWarpScaleChanged || bWeftScaleChanged)
		{
			// Need to update distances
			UpdateDists();
		}
	}
}

void FPBDEdgeSpringConstraints::ResetRestLengths(const TConstArrayView<FSolverVec3>& Positions)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPBDSpringConstraints_ResetRestLengths);

	TArrayView<FSolverReal> DistsToCompute = bWarpWeftScaleEnabled ? TArrayView<FSolverReal>(BaseDists) : TArrayView<FSolverReal>(Dists);
	CalculateRestLengths(Positions, DistsToCompute);

	UpdateDists();
}

void FPBDEdgeSpringConstraints::UpdateDists()
{
	if (bWarpWeftScaleEnabled)
	{
		const bool WarpScaleHasWeightMap = WarpScale.HasWeightMap();
		const bool WeftScaleHasWeightMap = WeftScale.HasWeightMap();
		const FSolverReal WarpScaleNoMap = (FSolverReal)WarpScale;
		const FSolverReal WeftScaleNoMap = (FSolverReal)WeftScale;
		for (int32 ConstraintIndex = 0; ConstraintIndex < BaseDists.Num(); ++ConstraintIndex)
		{
			const FSolverReal WarpScaleValue = WarpScaleHasWeightMap ? WarpScale[ConstraintIndex] : WarpScaleNoMap;
			const FSolverReal WeftScaleValue = WeftScaleHasWeightMap ? WeftScale[ConstraintIndex] : WeftScaleNoMap;

			Dists[ConstraintIndex] = BaseDists[ConstraintIndex] * FMath::Sqrt(FMath::Square(WeftScaleValue * WarpWeftScaleBaseMultipliers[ConstraintIndex][0]) + FMath::Square(WarpScaleValue * WarpWeftScaleBaseMultipliers[ConstraintIndex][1]));
		}
	}
}

void FPBDBendingSpringConstraints::SetProperties(
	const FCollectionPropertyConstFacade& PropertyCollection,
	const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps)
{
	if (IsBendingSpringStiffnessMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedValue(GetWeightedFloatBendingSpringStiffness(PropertyCollection));
		if (IsBendingSpringStiffnessStringDirty(PropertyCollection))
		{
			const FString& WeightMapName = GetBendingSpringStiffnessString(PropertyCollection);
			Stiffness = FPBDStiffness(
				WeightedValue,
				WeightMaps.FindRef(WeightMapName),
				TConstArrayView<TVec2<int32>>(Constraints),
				ParticleOffset,
				ParticleCount);
		}
		else
		{
			Stiffness.SetWeightedValue(WeightedValue);
		}
	}
}

} // End namespace Chaos::Softs
