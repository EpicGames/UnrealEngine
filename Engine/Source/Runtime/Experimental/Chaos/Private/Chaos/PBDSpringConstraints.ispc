// Copyright Epic Games, Inc. All Rights Reserved.

#define EXPLICIT_VECTOR4 1

#include "Math/Vector.isph"
#include "Chaos/PBDSofts.isph"

static inline float SafeNormalize(FVector3f &Direction)
{
	const float SizeSquared = VectorSizeSquared(Direction);
	const float Size = sqrt(SizeSquared);
	Direction = VectorSelect((SizeSquared < FLOAT_KINDA_SMALL_NUMBER), FloatForwardVector, Direction / Size);
	return (SizeSquared < FLOAT_KINDA_SMALL_NUMBER) ? FLOAT_ZERO : Size;
}

export void ApplySpringConstraints(uniform FVector4f PandInvM[],
									const uniform FIntVector2 Constraints[],
									const uniform float Dists[],
									const uniform float Stiffness,
									const uniform int32 NumConstraints)
{
	foreach(i = 0 ... NumConstraints)
	{
		const varying FIntVector2 Constraint = VectorLoad(&Constraints[extract(i,0)]);
		const varying int32 i1 = Constraint.V[0];
		const varying int32 i2 = Constraint.V[1];

		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[i1]);
		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[i2]);

		varying FVector3f P1, P2;
		varying float InvM1, InvM2;
		UnzipPandInvM(PandInvM1, P1, InvM1);
		UnzipPandInvM(PandInvM2, P2, InvM2);

		FVector3f Direction = P1 - P2;
		const float Distance = SafeNormalize(Direction);

		const varying float Dist = Dists[i];

		const FVector3f InnerDelta = (Distance - Dist) * Direction;

		const float CombinedInvMass = InvM2 + InvM1;
		const FVector3f ComputedDelta = Stiffness * InnerDelta / CombinedInvMass;

		const FVector3f Delta = VectorSelect((InvM2 == FLOAT_ZERO && InvM1 == FLOAT_ZERO), FloatZeroVector, ComputedDelta);

		if (InvM1 > FLOAT_ZERO)
		{
			VectorScatter(&PandInvM[i1], SetVector4( P1 - (InvM1 * Delta), InvM1 ));
		}
		if (InvM2 > FLOAT_ZERO)
		{
			VectorScatter(&PandInvM[i2], SetVector4( P2 + (InvM2 * Delta), InvM2 ));
		}
	}
}

export void ApplySpringConstraintsWithWeightMaps(uniform FVector4f PandInvM[],
									const uniform FIntVector2 Constraints[],
									const uniform float Dists[],
									const uniform uint8 StiffnessIndices[],
									const uniform float StiffnessTable[],
									const uniform int32 NumConstraints)
{

	foreach(i = 0 ... NumConstraints)
	{
		const varying FIntVector2 Constraint = VectorLoad(&Constraints[extract(i,0)]);
		const varying int32 i1 = Constraint.V[0];
		const varying int32 i2 = Constraint.V[1];

		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[i1]);
		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[i2]);

		varying FVector3f P1, P2;
		varying float InvM1, InvM2;
		UnzipPandInvM(PandInvM1, P1, InvM1);
		UnzipPandInvM(PandInvM2, P2, InvM2);

		FVector3f Direction = P1 - P2;
		const float Distance = SafeNormalize(Direction);

		const varying float Dist = Dists[i];
		const varying uint8 StiffnessIndex = StiffnessIndices[i];

		const FVector3f InnerDelta = (Distance - Dist) * Direction;

		#pragma ignore warning(perf)
		const varying float Stiffness = StiffnessTable[StiffnessIndex];

		const float CombinedInvMass = InvM2 + InvM1;
		const FVector3f ComputedDelta = Stiffness * InnerDelta / CombinedInvMass;

		const FVector3f Delta = VectorSelect((InvM2 == FLOAT_ZERO && InvM1 == FLOAT_ZERO), FloatZeroVector, ComputedDelta);

		if (InvM1 > FLOAT_ZERO)
		{
			VectorScatter(&PandInvM[i1], SetVector4( P1 - (InvM1 * Delta), InvM1 ));
		}
		if (InvM2 > FLOAT_ZERO)
		{
			VectorScatter(&PandInvM[i2], SetVector4( P2 + (InvM2 * Delta), InvM2 ));
		}
	}
}
