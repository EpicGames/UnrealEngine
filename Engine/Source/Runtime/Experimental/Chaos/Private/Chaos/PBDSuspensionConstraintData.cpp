// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/PBDSuspensionConstraintData.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "Chaos/PhysicsObjectInterface.h"

namespace Chaos
{
	FSuspensionConstraint::FSuspensionConstraint()
		: FConstraintBase(EConstraintType::SuspensionConstraintType)

	{
	}

	void FSuspensionConstraint::SetParticleProxy(IPhysicsProxyBase* InParticleProxy)
	{
		check(InParticleProxy);

		SuspensionProxy.Modify(/*bInvalidate=*/true, DirtyFlags, Proxy, [InParticleProxy](FProxyBaseProperty& Data)
			{
				ensure(InParticleProxy->GetType() == EPhysicsProxyType::SingleParticleProxy); // legacy, only worked with SingleParticleProxy
				Data.Proxy = InParticleProxy;
			});

		// This should work fine since this is a legacy endpoint and was primarily only used with single particle physics proxies.
		SuspensionBody.Modify(/*bInvalidate=*/true, DirtyFlags, Proxy, [InParticleProxy](FPhysicsObjectProperty& Data)
			{
				Data.PhysicsBody = (InParticleProxy->GetType() == EPhysicsProxyType::SingleParticleProxy) ? static_cast<FSingleParticlePhysicsProxy*>(InParticleProxy)->GetPhysicsObject() : nullptr;
			});
	}

	void FSuspensionConstraint::SetPhysicsBody(FPhysicsObjectHandle& InBody)
	{
		check(InBody);

		SuspensionProxy.Modify(/*bInvalidate=*/true, DirtyFlags, Proxy, [InBody](FProxyBaseProperty& Data)
			{
				Data.Proxy = Chaos::FPhysicsObjectInterface::GetProxy({ &InBody, 1 });
			});

		SuspensionBody.Modify(/*bInvalidate=*/true, DirtyFlags, Proxy, [InBody](FPhysicsObjectProperty& Data)
			{
				Data.PhysicsBody = InBody;
			});
	}

	void FSuspensionConstraint::SetGroundBody(FPhysicsObjectHandle& InBody)
	{
		check(InBody);

		GroundBody.Modify(/*bInvalidate=*/true, DirtyFlags, Proxy, [InBody](FPhysicsObjectProperty& Data)
			{
				Data.PhysicsBody = InBody;
			});
	}

} // Chaos
