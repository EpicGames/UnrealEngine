// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/PBDSuspensionConstraints.h"
#include "Chaos/Collision/PBDCollisionSolver.h"
#include "Chaos/Island/IslandManager.h"
#include "Chaos/DebugDrawQueue.h"

bool bChaos_Suspension_Spring_Enabled = true;
FAutoConsoleVariableRef CVarChaosSuspensionSpringEnabled(TEXT("p.Chaos.Suspension.Spring.Enabled"), bChaos_Suspension_Spring_Enabled, TEXT("Enable/Disable Spring part of suspension constraint"));

bool bChaos_Suspension_Hardstop_Enabled = true;
FAutoConsoleVariableRef CVarChaosSuspensionHardstopEnabled(TEXT("p.Chaos.Suspension.Hardstop.Enabled"), bChaos_Suspension_Hardstop_Enabled, TEXT("Enable/Disable Hardstop part of suspension constraint"));

bool bChaos_Suspension_VelocitySolve = true;
FAutoConsoleVariableRef CVarChaosSuspensionVelocitySolve(TEXT("p.Chaos.Suspension.VelocitySolve"), bChaos_Suspension_VelocitySolve, TEXT("Enable/Disable VelocitySolve"));

float Chaos_Suspension_MaxPushoutVelocity = 100.f;
FAutoConsoleVariableRef CVarChaosSuspensionMaxPushoutVelocity(TEXT("p.Chaos.Suspension.MaxPushoutVelocity"), Chaos_Suspension_MaxPushoutVelocity, TEXT("Chaos Suspension Max Pushout Velocity Value"));

float Chaos_Suspension_MaxPushout = 5.f;
FAutoConsoleVariableRef CVarChaosSuspensionMaxPushout(TEXT("p.Chaos.Suspension.MaxPushout"), Chaos_Suspension_MaxPushout, TEXT("Chaos Suspension Max Pushout Value"));

float Chaos_Suspension_SlopeThreshold = 0.707f;	// = Cos(SlopeAngle)
FAutoConsoleVariableRef CVarChaosSuspensionSlopeThreshold(TEXT("p.Chaos.Suspension.SlopeThreshold"), Chaos_Suspension_SlopeThreshold, TEXT("Slope threshold below which the anti-slide on slope mechanism is employed, value = Cos(AlopeAngle), i.e. for 50 degree slope = 0.6428, 30 degree slope = 0.866"));

float Chaos_Suspension_SlopeSpeedThreshold = 1.0f; // MPH
FAutoConsoleVariableRef CVarChaosSuspensionSlopeSpeedThreshold(TEXT("p.Chaos.Suspension.SlopeSpeedThreshold"), Chaos_Suspension_SlopeSpeedThreshold, TEXT("Speed below which the anti-slide on slope mechanism is fully employed"));

float Chaos_Suspension_SlopeSpeedBlendThreshold = 10.0f; // MPH
FAutoConsoleVariableRef CVarChaosSuspensionSlopeSpeedBlendThreshold(TEXT("p.Chaos.Suspension.SlopeSpeedBlendThreshold"), Chaos_Suspension_SlopeSpeedBlendThreshold, TEXT("Speed below which the anti-slide on slope blend mechanism starts"));

#if CHAOS_DEBUG_DRAW
bool bChaos_Suspension_DebugDraw_Hardstop = false;
FAutoConsoleVariableRef CVarChaosSuspensionDebugDrawHardstop(TEXT("p.Chaos.Suspension.DebugDraw.Hardstop"), bChaos_Suspension_DebugDraw_Hardstop, TEXT("Debug draw suspension hardstop manifold"));
#endif

namespace Chaos
{
	FPBDSuspensionConstraintHandle::FPBDSuspensionConstraintHandle(FConstraintContainer* InConstraintContainer, int32 InConstraintIndex) : 
		TIndexedContainerConstraintHandle<FPBDSuspensionConstraints>(InConstraintContainer, InConstraintIndex)
	{
	}

	const FPBDSuspensionSettings& FPBDSuspensionConstraintHandle::GetSettings() const
	{
		return ConcreteContainer()->GetSettings(ConstraintIndex);
	}

	FPBDSuspensionSettings& FPBDSuspensionConstraintHandle::GetSettings()
	{
		return ConcreteContainer()->GetSettings(ConstraintIndex);
	}

	void FPBDSuspensionConstraintHandle::SetSettings(const FPBDSuspensionSettings& Settings)
	{
		ConcreteContainer()->SetSettings(ConstraintIndex, Settings);
	}

	FParticlePair FPBDSuspensionConstraintHandle::GetConstrainedParticles() const
	{
		return ConcreteContainer()->GetConstrainedParticles(ConstraintIndex);
	}

	FPBDSuspensionConstraints::FConstraintContainerHandle* FPBDSuspensionConstraints::AddConstraint(TGeometryParticleHandle<FReal, 3>* Particle, const FVec3& InSuspensionLocalOffset, const FPBDSuspensionSettings& InConstraintSettings)
	{
		int32 NewIndex = ConstrainedParticles.Num();
		ConstrainedParticles.Add(Particle);
		GroundParticles.Add(nullptr);
		SuspensionLocalOffset.Add(InSuspensionLocalOffset);
		ConstraintSettings.Add(InConstraintSettings);
		ConstraintResults.AddDefaulted();
		ConstraintEnabledStates.Add(true); // Note: assumes always enabled on creation
		ConstraintSolverBodies.Add({ nullptr, nullptr });
		CollisionSolvers.Add(Private::FPBDCollisionSolver());
		CollisionSolvers[CollisionSolvers.Num() - 1].MakeInitialized();
		CollisionSolverManifoldPoints.Add(Private::FPBDCollisionSolverManifoldPoint());

		Handles.Add(HandleAllocator.AllocHandle(this, NewIndex));
		return Handles[NewIndex];
	}


	void FPBDSuspensionConstraints::RemoveConstraint(int ConstraintIndex)
	{
		FConstraintContainerHandle* ConstraintHandle = Handles[ConstraintIndex];
		if (ConstraintHandle != nullptr)
		{
			if (ConstrainedParticles[ConstraintIndex])
			{
				ConstrainedParticles[ConstraintIndex]->RemoveConstraintHandle(ConstraintHandle);
			}

			// Release the handle for the freed constraint
			HandleAllocator.FreeHandle(ConstraintHandle);
			Handles[ConstraintIndex] = nullptr;
		}

		// Swap the last constraint into the gap to keep the array packed
		ConstrainedParticles.RemoveAtSwap(ConstraintIndex);
		GroundParticles.RemoveAtSwap(ConstraintIndex);
		SuspensionLocalOffset.RemoveAtSwap(ConstraintIndex);
		ConstraintSettings.RemoveAtSwap(ConstraintIndex);
		ConstraintResults.RemoveAtSwap(ConstraintIndex);
		ConstraintEnabledStates.RemoveAtSwap(ConstraintIndex);
		ConstraintSolverBodies.RemoveAtSwap(ConstraintIndex);
		CollisionSolvers.RemoveAtSwap(ConstraintIndex);
		CollisionSolverManifoldPoints.RemoveAtSwap(ConstraintIndex);

		Handles.RemoveAtSwap(ConstraintIndex);

		// Update the handle for the constraint that was moved
		if (ConstraintIndex < Handles.Num())
		{
			SetConstraintIndex(Handles[ConstraintIndex], ConstraintIndex);
		}
	}

	void FPBDSuspensionConstraints::DisconnectConstraints(const TSet<TGeometryParticleHandle<FReal, 3>*>& RemovedParticles)
	{
		for (FGeometryParticleHandle* RemovedParticle : RemovedParticles)
		{
			for (FConstraintHandle* ConstraintHandle : RemovedParticle->ParticleConstraints())
			{
				if (FPBDSuspensionConstraintHandle* SuspensionHandle = ConstraintHandle->As<FPBDSuspensionConstraintHandle>())
				{
					SuspensionHandle->SetEnabled(false); // constraint lifespan is managed by the proxy

					int ConstraintIndex = SuspensionHandle->GetConstraintIndex();
					if (ConstraintIndex != INDEX_NONE)
					{
						if (ConstrainedParticles[ConstraintIndex] == RemovedParticle)
						{
							ConstrainedParticles[ConstraintIndex] = nullptr;
							GroundParticles[ConstraintIndex] = nullptr;
						}
					}
				}
			}
		}
	}

	void FPBDSuspensionConstraints::SetConstraintEnabled(int32 ConstraintIndex, bool bEnabled)
	{
		if (ConstraintIndex < ConstrainedParticles.Num())
		{
			const FGenericParticleHandle Particle = FGenericParticleHandle(ConstrainedParticles[ConstraintIndex]);

			if (bEnabled)
			{
				// only enable constraint if the particle is valid and not disabled
				if (Particle->Handle() != nullptr && !Particle->Disabled())
				{
					ConstraintEnabledStates[ConstraintIndex] = true;
				}
			}
			else
			{
				// desirable to allow disabling no matter what state the endpoint
				ConstraintEnabledStates[ConstraintIndex] = false;
			}
		}
	}

	void FPBDSuspensionConstraints::SetGroundParticle(int32 ConstraintIndex, Chaos::FGeometryParticleHandle* InGroundParticle)
	{
		if (GroundParticles.IsValidIndex(ConstraintIndex))
		{
			if (GroundParticles[ConstraintIndex] != InGroundParticle)
			{
				if (FConstraintContainerHandle* Constraint = GetConstraintHandle(ConstraintIndex))
				{
					Constraint->SetGroundParticleChanged();
				}
			}

			GroundParticles[ConstraintIndex] = InGroundParticle;
		}
	}

	void FPBDSuspensionConstraints::AddConstraintsToGraph(Private::FPBDIslandManager& IslandManager)
	{
		for (FPBDSuspensionConstraintHandle* Constraint : GetConstraints())
		{
			const bool bIsInGraph = Constraint->IsInConstraintGraph();
			const bool bShouldBeInGraph = Constraint->IsEnabled();
			const bool bShouldUpdateGraph = Constraint->HasGroundParticleChanged();

			if (bShouldBeInGraph && !bIsInGraph)
			{
				IslandManager.AddConstraint(ContainerId, Constraint, Constraint->GetConstrainedParticles());
			}
			else if (bIsInGraph && !bShouldBeInGraph)
			{
				IslandManager.RemoveConstraint(Constraint);
			}
			else if (bIsInGraph && bShouldUpdateGraph)
			{
				IslandManager.RemoveConstraint(Constraint);
				IslandManager.AddConstraint(ContainerId, Constraint, Constraint->GetConstrainedParticles());
				Constraint->ClearGroundParticleChanged();
			}
		}
	}

	void FPBDSuspensionConstraints::AddBodies(FSolverBodyContainer& SolverBodyContainer)
	{
		for (int32 ConstraintIndex = 0; ConstraintIndex < NumConstraints(); ++ConstraintIndex)
		{
			AddBodies(ConstraintIndex, SolverBodyContainer);
		}
	}

	void FPBDSuspensionConstraints::GatherInput(const FReal Dt)
	{
		for (int32 ConstraintIndex = 0; ConstraintIndex < NumConstraints(); ++ConstraintIndex)
		{
			GatherInput(ConstraintIndex, Dt);
		}
	}

	void FPBDSuspensionConstraints::ScatterOutput(const FReal Dt)
	{
		for (int32 ConstraintIndex = 0; ConstraintIndex < NumConstraints(); ++ConstraintIndex)
		{
			ScatterOutput(ConstraintIndex, Dt);
		}
	}

	void FPBDSuspensionConstraints::ApplyPositionConstraints(const FReal Dt, const int32 It, const int32 NumIts)
	{
		for (int32 ConstraintIndex = 0; ConstraintIndex < NumConstraints(); ++ConstraintIndex)
		{
			ApplyPositionConstraint(ConstraintIndex, Dt, It, NumIts);
		}
	}

	void FPBDSuspensionConstraints::ApplyVelocityConstraints(const FReal Dt, const int32 It, const int32 NumIts)
	{
		for (int32 ConstraintIndex = 0; ConstraintIndex < NumConstraints(); ++ConstraintIndex)
		{
			ApplyVelocityConstraint(ConstraintIndex, Dt, It, NumIts);
		}
	}

	void FPBDSuspensionConstraints::AddBodies(const TArrayView<int32>& ConstraintIndices, FSolverBodyContainer& SolverBodyContainer)
	{
		for (int32 ConstraintIndex : ConstraintIndices)
		{
			AddBodies(ConstraintIndex, SolverBodyContainer);
		}
	}

	void FPBDSuspensionConstraints::GatherInput(const TArrayView<int32>& ConstraintIndices, const FReal Dt)
	{
		for (int32 ConstraintIndex : ConstraintIndices)
		{
			GatherInput(ConstraintIndex, Dt);
		}
	}

	void FPBDSuspensionConstraints::ScatterOutput(const TArrayView<int32>& ConstraintIndices, const FReal Dt)
	{
		for (int32 ConstraintIndex : ConstraintIndices)
		{
			ScatterOutput(ConstraintIndex, Dt);
		}
	}

	void FPBDSuspensionConstraints::ApplyPositionConstraints(const TArrayView<int32>& ConstraintIndices, const FReal Dt, const int32 It, const int32 NumIts)
	{
		for (int32 ConstraintIndex : ConstraintIndices)
		{
			ApplyPositionConstraint(ConstraintIndex, Dt, It, NumIts);
		}
	}

	void FPBDSuspensionConstraints::ApplyVelocityConstraints(const TArrayView<int32>& ConstraintIndices, const FReal Dt, const int32 It, const int32 NumIts)
	{
		for (int32 ConstraintIndex : ConstraintIndices)
		{
			ApplyVelocityConstraint(ConstraintIndex, Dt, It, NumIts);
		}
	}

	void FPBDSuspensionConstraints::AddBodies(const int32 ConstraintIndex, FSolverBodyContainer& SolverBodyContainer)
	{
		// FindOrAdd expects a valid pointer
		if (ConstrainedParticles[ConstraintIndex])
		{
			if (GroundParticles[ConstraintIndex])
			{
				ConstraintSolverBodies[ConstraintIndex] =
				{
					SolverBodyContainer.FindOrAdd(ConstrainedParticles[ConstraintIndex]),
					SolverBodyContainer.FindOrAdd(GroundParticles[ConstraintIndex])
				};
			}
			else
			{
				ConstraintSolverBodies[ConstraintIndex] =
				{
					SolverBodyContainer.FindOrAdd(ConstrainedParticles[ConstraintIndex]),
					nullptr
				};
			}
		}
	}

	void FPBDSuspensionConstraints::GatherInput(const int32 ConstraintIndex, const FReal Dt)
	{
		using namespace Private;

		ConstraintResults[ConstraintIndex].Reset();

		// Hard-stop Collision Manifold Generation
		{

			FPBDCollisionSolver* Solver = &CollisionSolvers[ConstraintIndex];
			Solver->Reset(&CollisionSolverManifoldPoints[ConstraintIndex], 1);
			Solver->SetStiffness(1);
			Solver->SetHardContact();

			// this is the case if the raycast has not found a body to hit - can we disable constraint in that instance rather than running
			if (ConstraintSolverBodies[ConstraintIndex][1] == nullptr)
			{
				return;
			}

			check(ConstraintSolverBodies[ConstraintIndex][0] != nullptr);
			check(ConstraintSolverBodies[ConstraintIndex][1] != nullptr);

			FSolverBody* Body0 = ConstraintSolverBodies[ConstraintIndex][0];	// vehicle chassis			
			FSolverBody* Body1 = ConstraintSolverBodies[ConstraintIndex][1];	// body that the wheel is touching

			bool bHitSelf = (Body0 == Body1);

			const FPBDSuspensionSettings& Setting = ConstraintSettings[ConstraintIndex];

			if (Body0->IsDynamic() && !bHitSelf && Setting.Enabled)
			{
				const FVec3& T = Setting.Target;

				// \todo(chaos): we can cache the CoM-relative connector once per frame rather than recalculate per iteration
				// (we should not be accessing particle state in the solver methods, although this one actually is ok because it only uses frame constrants)
				const FGenericParticleHandle Particle = ConstrainedParticles[ConstraintIndex];
				const FVec3& SuspensionActorOffset = SuspensionLocalOffset[ConstraintIndex];
				const FVec3 SuspensionCoMOffset = Particle->RotationOfMass().UnrotateVector(SuspensionActorOffset - Particle->CenterOfMass());
				const FVec3 SuspensionCoMAxis = Particle->RotationOfMass().UnrotateVector(Setting.Axis);
			
				const FRotation3 BodyQ = Body0->CorrectedQ();
				const FVec3 BodyP = Body0->CorrectedP();
				const FVec3 WorldSpaceX = BodyQ.RotateVector(SuspensionCoMOffset) + BodyP;
				FVec3 AxisWorld = BodyQ.RotateVector(SuspensionCoMAxis);
				FReal Distance = FVec3::DotProduct(WorldSpaceX - T, AxisWorld);

				const FVec3 WorldArm0 = BodyQ.RotateVector(SuspensionCoMOffset);
				const FVec3 WorldArm1 = WorldSpaceX - Body1->CorrectedP();

				// The hard-stop can only apply correction perpendicular to the surface
				const FVec3& WorldContactNormal = Setting.Normal;

				const FReal HardStopDistance = Setting.MinLength - Distance;
				const FReal WorldContactDeltaNormal = FVec3::DotProduct(HardStopDistance * AxisWorld, WorldContactNormal);

				Solver->SetSolverBodies(*Body0, *Body1);
				Solver->SolverBody0().Init();
				Solver->SolverBody1().Init();
				Solver->SetFriction(0, 0, 0, 0);

#if CHAOS_DEBUG_DRAW
				if (bChaos_Suspension_DebugDraw_Hardstop)
				{
					FVec3 Body0Center = WorldSpaceX;
					FReal Radius = 30;
					FDebugDrawQueue::GetInstance().DrawDebugCircle(Body0Center, Radius, 60, FColor::Yellow, false, -1.0f, 0, 3, FVector(1, 0, 0), FVector(0, 1, 0), false);
					FString TextOut = FString::Format(TEXT("{0}"), { WorldContactDeltaNormal });
					FDebugDrawQueue::GetInstance().DrawDebugString(Body0Center + FVec3(0, 50, 50), TextOut, nullptr, FColor::White, -1.f, true, 1.0f);
				}
#endif

				// inject a manifold for our suspension Hard-stop - behaves like a regular friction-less collision, prevents the vehicle chassis from ever hitting the ground
				Solver->AddManifoldPoint();
				Solver->InitManifoldPoint(
					0,										// ManifoldIndex
					FSolverReal(Dt),						// Delta Time
					FSolverVec3(WorldArm0),					// RelativeContactPosition0,
					FSolverVec3(WorldArm1),					// RelativeContactPosition1,
					FSolverVec3(WorldContactNormal),		// WorldContactNormal
					FSolverVec3(0, 0, 0),					// WorldContactTangentU,
					FSolverVec3(0, 0, 0),					// WorldContactTangentV,
					FSolverReal(-WorldContactDeltaNormal),	// WorldContactDeltaNormal
					FSolverReal(0),							// WorldContactDeltaTangentU,
					FSolverReal(0),							// WorldContactDeltaTangentV
					FSolverReal(0)							// WorldTargetContactVelocityNormal
					);
				Solver->FinalizeManifold();
			}
		}

	}

	void FPBDSuspensionConstraints::ScatterOutput(const int32 ConstraintIndex, FReal Dt)
	{
		using namespace Private;

		if ((CollisionSolvers.Num() > 0) && (CollisionSolvers[ConstraintIndex].NumManifoldPoints() > 0))
		{
			const FPBDCollisionSolverManifoldPoint& ManifoldPoint = CollisionSolvers[ConstraintIndex].GetManifoldPoint(0);
			ConstraintResults[ConstraintIndex].HardStopNetPushOut = ManifoldPoint.NetPushOutNormal * ManifoldPoint.ContactNormal;
			ConstraintResults[ConstraintIndex].HardStopNetImpulse = ManifoldPoint.NetImpulseNormal * ManifoldPoint.ContactNormal;
		}

		ConstraintSolverBodies[ConstraintIndex] =
		{
			nullptr,
			nullptr
		};

		CollisionSolvers[ConstraintIndex].Reset(nullptr, 0);
	}

	void FPBDSuspensionConstraints::ApplyPositionConstraint(const int32 ConstraintIndex, const FReal Dt, const int32 It, const int32 NumIts)
	{
		using namespace Private;
			
		if (bChaos_Suspension_Hardstop_Enabled)
		{
			// Suspension Hardstop
			const FPBDSuspensionSettings& Setting = ConstraintSettings[ConstraintIndex];
			if (Setting.Enabled)
			{
				FPBDCollisionSolver* CollisionSolver = &CollisionSolvers[ConstraintIndex];
				if ((CollisionSolver != nullptr) && (CollisionSolver->NumManifoldPoints() > 0))
				{
					FReal MaxPushoutValue = FMath::Min(Chaos_Suspension_MaxPushout, Chaos_Suspension_MaxPushoutVelocity * Dt);
					CollisionSolver->SolvePositionNoFriction(FSolverReal(Dt), FSolverReal(MaxPushoutValue));
				}
			}
		}

		if (bChaos_Suspension_Spring_Enabled)
		{
			// Suspension Spring
			ApplySingle(ConstraintIndex, Dt);
		}
	}

	void FPBDSuspensionConstraints::ApplyVelocityConstraint(const int32 ConstraintIndex, const FReal Dt, const int32 It, const int32 NumIts)
	{
		using namespace Private;

		if (bChaos_Suspension_Hardstop_Enabled && bChaos_Suspension_VelocitySolve)
		{
			// Suspension Hardstop
			const FPBDSuspensionSettings& Setting = ConstraintSettings[ConstraintIndex];
			if (Setting.Enabled)
			{
				FPBDCollisionSolver* CollisionSolver = &CollisionSolvers[ConstraintIndex];
				if ((CollisionSolver != nullptr) && (CollisionSolver->NumManifoldPoints() > 0))
				{
					CollisionSolver->SolveVelocity(FSolverReal(Dt), false);
				}
			}
		}
	}

	void FPBDSuspensionConstraints::ApplySingle(int32 ConstraintIndex, const FReal Dt)
	{
		// this is the case if the raycast has not found a body to hit - can we disable constraint in that instance rather than running
  		if (ConstraintSolverBodies[ConstraintIndex][1] == nullptr)
		{
			return;
		}

		check(ConstraintSolverBodies[ConstraintIndex][0] != nullptr);
		check(ConstraintSolverBodies[ConstraintIndex][1] != nullptr);
		FSolverBody& Body0 = *ConstraintSolverBodies[ConstraintIndex][0];
		FSolverBody& Body1 = *ConstraintSolverBodies[ConstraintIndex][1];

		bool bHitSelf = (ConstraintSolverBodies[ConstraintIndex][0] == ConstraintSolverBodies[ConstraintIndex][1]);

		const FPBDSuspensionSettings& Setting = ConstraintSettings[ConstraintIndex];
		FPBDSuspensionResults& Results = ConstraintResults[ConstraintIndex];

		if (Body0.IsDynamic() && !bHitSelf && Setting.Enabled)
		{
			const FVec3& T = Setting.Target;

			// \todo(chaos): we can cache the CoM-relative connector once per frame rather than recalculate per iteration
			// (we should not be accessing particle state in the solver methods, although this one actually is ok because it only uses frame constrants)
			const FGenericParticleHandle Particle = ConstrainedParticles[ConstraintIndex];
			const FVec3& SuspensionActorOffset = SuspensionLocalOffset[ConstraintIndex];
			const FVec3 SuspensionCoMOffset = Particle->RotationOfMass().UnrotateVector(SuspensionActorOffset - Particle->CenterOfMass());
			const FVec3 SuspensionCoMAxis = Particle->RotationOfMass().UnrotateVector(Setting.Axis);

			// @todo(chaos): use linearized error calculation
			const FRotation3 BodyQ = Body0.CorrectedQ();
			const FVec3 BodyP = Body0.CorrectedP();
			const FVec3 WorldSpaceX = BodyQ.RotateVector(SuspensionCoMOffset) + BodyP;
			const FRotation3 BodyQ2 = Body1.CorrectedQ();
			const FVec3 BodyP2 = Body1.CorrectedP();

			FVec3 AxisWorld = BodyQ.RotateVector(SuspensionCoMAxis);

			FVec3 SurfaceNormal = Setting.Normal;

			const float MPHToCmS = 100000.f / 2236.94185f;
			const float SpeedThreshold = Chaos_Suspension_SlopeSpeedThreshold * MPHToCmS;
			const float SpeedBlendThreshold = Chaos_Suspension_SlopeSpeedBlendThreshold * MPHToCmS;

			// Ingeniously blends the surface normal at low speeds to stop vehicles sliding slowly down steep slopes
			if (SurfaceNormal.Z > Chaos_Suspension_SlopeThreshold)
			{
				if (Body0.V().SquaredLength() < (SpeedThreshold * SpeedThreshold))
				{
					SurfaceNormal = FVec3(0.f, 0.f, 1.f);
				}
				else
				{
					const FReal Speed = FMath::Abs(Body0.V().Length());
					if (Speed < SpeedBlendThreshold)
					{
						SurfaceNormal = FMath::Lerp(FVec3(0.f, 0.f, 1.f), SurfaceNormal, Speed / SpeedBlendThreshold);
					}
				}
			}

			FReal Distance = FVec3::DotProduct(WorldSpaceX - T, AxisWorld);
			if (Distance >= Setting.MaxLength)
			{
				// do nothing since the target point is further than the longest extension of the suspension spring
				Results.Length = Setting.MaxLength;
				return;
			}

			FVec3 DX = FVec3::ZeroVector;

			// Require the velocity at the WorldSpaceX position - not the velocity of the particle origin
			// NOTE: We are in the position solve phase and velocty is not updated. We must use the implicit 
			// velocity in the damping calculation
			// @todo(chaos): consider moving the damping term to the velocity solve phase
			const FVec3 Diff = WorldSpaceX - BodyP;
			const FVec3 V = FVec3::CalculateVelocity(Body0.X(), BodyP, Dt);
			const FVec3 W = FRotation3::CalculateAngularVelocity(Body0.R(), BodyQ, Dt);
			FVec3 ArmVelocity = V - FVec3::CrossProduct(Diff, W);


			if (Distance < Setting.MinLength)
			{
				// target point distance is less at min compression limit 
				// - apply distance constraint to try keep a valid min limit
				//FVec3 Ts = WorldSpaceX + AxisWorld * (Setting.MinLength - Distance);
				//DX = (Ts - WorldSpaceX) * Setting.HardstopStiffness;

				Distance = Setting.MinLength;

				//if (PointVelocityAlongAxis < 0.0f)
				//{
				//	const FVec3 SpringVelocity = PointVelocityAlongAxis * AxisWorld;
				//	DX -= SpringVelocity * Setting.HardstopVelocityCompensation;
				//	PointVelocityAlongAxis = 0.0f; //this Dx will cancel velocity, so don't pass PointVelocityAlongAxis on to suspension force calculation 
				//}
			}

			{
				// then the suspension force on top

				FReal DLambda = 0.f;
				{
					// #todo: Preload, better scaled spring damping like other suspension 0 -> 1 range
					const FReal AxisDotNormal = FVec3::DotProduct(AxisWorld, SurfaceNormal);
					FReal SpringCompression = AxisDotNormal * (Setting.MaxLength - Distance) /*+ Setting.SpringPreload*/;
					FReal SpringVelocity = FVec3::DotProduct(ArmVelocity, SurfaceNormal);

					// @todo(chaos): this is not using the correct effective mass
					const bool bAccelerationMode = false;
					const FReal SpringMassScale = (bAccelerationMode) ? FReal(1) / Body0.InvM() : FReal(1);
					const FReal S = SpringMassScale * Setting.SpringStiffness * Dt * Dt;
					const FReal D = SpringMassScale * Setting.SpringDamping * Dt;
					
					// @todo(chaos): add missing XPBD term to get iteration-count independent behaviour
					DLambda = (S * SpringCompression - D * SpringVelocity);

					// Suspension springs cannot apply downward forces on the body
					if (DLambda < 0)
					{
						DLambda = 0;
					}

					DX += DLambda * SurfaceNormal;
				}
			}

			const FVec3 Arm = WorldSpaceX - BodyP;

			const FVec3 DP = Body0.InvM() * DX;
			const FVec3 DR = Body0.InvI() * FVec3::CrossProduct(Arm, DX);
			Body0.ApplyTransformDelta(DP, DR);
			Body0.UpdateRotationDependentState();

			if (Body1.IsDynamic())
			{
				const FVec3 Arm2 = WorldSpaceX - BodyP2;
				const FVec3 DP2 = Body1.InvM() * -DX;
				const FVec3 DR2 = Body1.InvI() * FVec3::CrossProduct(Arm2, -DX);
				Body1.ApplyTransformDelta(DP2, DR2);
				Body1.UpdateRotationDependentState();
			}

			Results.NetPushOut += DX;
			Results.Length = Distance;
		}
	}
	
	template class TIndexedContainerConstraintHandle<FPBDSuspensionConstraints>;


}
