// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/PBDTriangleMeshCollisions.h"
#include "Chaos/Plane.h"
#include "Chaos/PBDSoftsSolverParticles.h"
#include "Chaos/SoftsSolverParticlesRange.h"
#include "Chaos/Triangle.h"
#include "Chaos/TriangleCollisionPoint.h"
#include "Chaos/TriangleMesh.h"
#include "Chaos/Framework/Parallel.h"
#include <atomic>

namespace Chaos::Softs {

struct FEdgeFaceIntersection
{
	int32 EdgeIndex;
	int32 FaceIndex;
	FSolverReal EdgeCoordinate;
	FSolverVec2 FaceCoordinate;
	FSolverVec3 FaceNormal;
	FSolverVec3 IntersectionPoint;
	bool bFlipGradient = false; // Only used when building global contours
};

// Returned array has NOT been shrunk
template<typename SpatialAccelerator, typename SolverParticlesOrRange>
static void FindEdgeFaceIntersections(const FTriangleMesh& TriangleMesh, const SpatialAccelerator& Spatial, const SolverParticlesOrRange& Particles, TArray<FEdgeFaceIntersection>& Intersections)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ChaosFPBDTriangleMeshCollisions_IntersectionQuery);

	const FSegmentMesh& SegmentMesh = TriangleMesh.GetSegmentMesh();
	const TArray<TVec2<int32>>& EdgeToFaces = TriangleMesh.GetEdgeToFaces();
	const TArray<TVec3<int32>>& Elements = TriangleMesh.GetElements();

	const int32 NumIntersectableEdges = SegmentMesh.GetNumElements();

	// Preallocate enough space for (more than) typical number of expected intersections.
	constexpr int32 PreallocatedIntersectionsPerEdge = 3;
	const int32 PreallocatedIntersectionsNum = PreallocatedIntersectionsPerEdge * NumIntersectableEdges;
	Intersections.Reset();
	Intersections.SetNumUninitialized(PreallocatedIntersectionsNum);
	std::atomic<int32> IntersectionIndex(0);

	// Extra intersections that require a lock to write to if you have more than PreallocatedIntersectionsPerEdge 
	TArray<FEdgeFaceIntersection> ExtraIntersections;
	FCriticalSection CriticalSection;
	PhysicsParallelFor(NumIntersectableEdges,
		[&Spatial, &TriangleMesh, &Particles, &SegmentMesh, &EdgeToFaces, &Elements, &IntersectionIndex, &Intersections, PreallocatedIntersectionsNum, &ExtraIntersections, &CriticalSection](int32 EdgeIndex)
		{
			TArray< TTriangleCollisionPoint<FSolverReal> > Result;

			const int32 EdgePointIndex0 = SegmentMesh.GetElements()[EdgeIndex][0];
			const int32 EdgePointIndex1 = SegmentMesh.GetElements()[EdgeIndex][1];
			const FSolverVec3& EdgePosition0 = Particles.X(EdgePointIndex0);
			const FSolverVec3& EdgePosition1 = Particles.X(EdgePointIndex1);
			// No faces can be kinematic, but edges can still be since a face is only kinematic if all 3 vertices are kinematic
			const bool bEdgeIsKinematic = Particles.InvM(EdgePointIndex0) == (FSolverReal)0. && Particles.InvM(EdgePointIndex1) == (FSolverReal)0.;
			if (bEdgeIsKinematic)
			{
				return;
			}

			if (TriangleMesh.EdgeIntersectionQuery(Spatial, static_cast<const TArrayView<const FSolverVec3>&>(Particles.XArray()), EdgeIndex, EdgePosition0, EdgePosition1,
				[&Elements, EdgePointIndex0, EdgePointIndex1, &Particles](int32 EdgeIndex, int32 TriangleIndex)
				{
					if (EdgePointIndex0 == Elements[TriangleIndex][0] || EdgePointIndex0 == Elements[TriangleIndex][1] || EdgePointIndex0 == Elements[TriangleIndex][2] ||
						EdgePointIndex1 == Elements[TriangleIndex][0] || EdgePointIndex1 == Elements[TriangleIndex][1] || EdgePointIndex1 == Elements[TriangleIndex][2])
					{
						return false;
					}

					return true;
				},
				Result))
			{

				for (int32 CollisionPointIndex = 0; CollisionPointIndex < Result.Num(); ++CollisionPointIndex)
				{
					const TTriangleCollisionPoint<FSolverReal>& CollisionPoint = Result[CollisionPointIndex];

					check(CollisionPoint.ContactType == TTriangleCollisionPoint<FSolverReal>::EContactType::EdgeFace);
					FEdgeFaceIntersection Intersection;
					Intersection.EdgeIndex = CollisionPoint.Indices[0];
					Intersection.FaceIndex = CollisionPoint.Indices[1];
					Intersection.EdgeCoordinate = CollisionPoint.Bary[0];
					Intersection.FaceCoordinate = { CollisionPoint.Bary[2], CollisionPoint.Bary[3] };
					Intersection.FaceNormal = CollisionPoint.Normal;
					Intersection.IntersectionPoint = CollisionPoint.Location;
					const int32 IndexToWrite = IntersectionIndex.fetch_add(1);
					if (IndexToWrite < PreallocatedIntersectionsNum)
					{
						Intersections[IndexToWrite] = Intersection;
					}
					else
					{
						CriticalSection.Lock();
						ExtraIntersections.Add(Intersection);
						CriticalSection.Unlock();
					}
				}
			}
		}
	);

	// Set Intersections Num to actual number found.
	const int32 IntersectionNum = FMath::Min(IntersectionIndex.load(), PreallocatedIntersectionsNum);
	Intersections.SetNum(IntersectionNum, EAllowShrinking::No);

	// Append any ExtraIntersections
	Intersections.Append(ExtraIntersections);
}

// Global intersection analysis (identifying global contours, flood filling)
namespace GIA
{
	struct FIntersectionContourTriangleSection
	{
		int32 LoopVertexLocalIndex = INDEX_NONE;
		int32 TriangleIndex = INDEX_NONE;

		TVec2<int32> CrossingEdgeLocalIndex = { INDEX_NONE,INDEX_NONE };

		FIntersectionContourTriangleSection() = default;
		FIntersectionContourTriangleSection(int32 InTriangleIndex)
			: TriangleIndex(InTriangleIndex)
		{}
	};

	struct FIntersectionContour
	{
		int8 BoundaryEdgeCount = 0;
		TArray<FIntersectionContourTriangleSection> Contour;		
	};

	struct FIntersectionContourPair
	{
		enum struct EClosedStatus : uint8
		{
			Open, // Can't color.
			SimpleClosed, // Expect to color both contours separately.
			LoopClosed, // Expect to have a single colorable contour.
			BoundaryClosed // Expect to have one colorable contour (two boundary ends), and one open contour.
		};
		EClosedStatus ClosedStatus = EClosedStatus::Open;
		int8 LoopVertexCount = 0; // Shared between ColorContours

		FIntersectionContour ColorContours[2];
		TArray<FEdgeFaceIntersection> Intersections;
		TArray<int32> ContourPointCurves[2]; // Track which curves correspond with which Contours. (Sometimes we generate multiple curves per contour)

		int8 NumContourEnds() const
		{
			return LoopVertexCount + ColorContours[0].BoundaryEdgeCount + ColorContours[1].BoundaryEdgeCount;
		}
	};

	// methods for building contours
	namespace ContourBuilding
	{
		// internal helper methods
		namespace __internal
		{
			static inline int32 GetLocalEdgeIndex(const TArray<TVec3<int32>>& FaceToEdges, const int32 FaceIndex, const int32 EdgeIndex)
			{
				for (int32 LocalIndex = 0; LocalIndex < 3; ++LocalIndex)
				{
					if (FaceToEdges[FaceIndex][LocalIndex] == EdgeIndex)
					{
						return LocalIndex;
					}
				}
				check(false); // we should never get here.
				return INDEX_NONE;
			}

			// returns local vertex index within FaceVertices
			// returns INDEX_NONE if no loop vertex found
			static inline int32 FindLoopVertex(const TVec3<int32>& FaceVertices, int32 OppositeVertexIndex)
			{
				for (int32 FaceLocalVertexIndex = 0; FaceLocalVertexIndex < 3; ++FaceLocalVertexIndex)
				{
					if (FaceVertices[FaceLocalVertexIndex] == OppositeVertexIndex)
					{
						return FaceLocalVertexIndex;
					}
				}
				return INDEX_NONE;
			}

			static inline bool IsNextIntersectionEdgeFaceFirstIntersection(const FEdgeFaceIntersection& FirstIntersection, const int32 EdgeFaceCrossingEdgeLocalIndex, const TVec3<int32>& FaceToEdges_EdgeFace, const int32 CurrIntersectionFaceIndex, int32& OutLocalEdgeIndex)
			{
				for (int32 LocalEdgeIndexOffset = 1; LocalEdgeIndexOffset < 3; ++LocalEdgeIndexOffset)
				{
					const int32 LocalEdgeIndex = (LocalEdgeIndexOffset + EdgeFaceCrossingEdgeLocalIndex) % 3;
					if (FirstIntersection.EdgeIndex == FaceToEdges_EdgeFace[LocalEdgeIndex] && FirstIntersection.FaceIndex == CurrIntersectionFaceIndex)
					{
						OutLocalEdgeIndex = LocalEdgeIndex;
						return true;
					}
				}

				return false;
			}

			static inline bool IsNextIntersectionFaceFirstIntersection(const FEdgeFaceIntersection& FirstIntersection, const TVec3<int32>& FaceToEdges_Face, const int32 EdgeFace, int32& OutLocalEdgeIndex)
			{
				for (int32 LocalEdgeIndex = 0; LocalEdgeIndex < 3; ++LocalEdgeIndex)
				{
					if (FirstIntersection.EdgeIndex == FaceToEdges_Face[LocalEdgeIndex] && FirstIntersection.FaceIndex == EdgeFace)
					{
						OutLocalEdgeIndex = LocalEdgeIndex;
						return true;
					}
				}

				return false;
			}

			static inline bool FindNextIntersectionEdgeFace(TMap<TVec2<int32>, FEdgeFaceIntersection>& IntersectionMap, const int32 EdgeFaceCrossingEdgeLocalIndex, const TVec3<int32>& FaceToEdges_EdgeFace, const int32 CurrIntersectionFaceIndex, FEdgeFaceIntersection& NextIntersection, int32& OutLocalEdgeIndex)
			{
				for (int32 LocalEdgeIndexOffset = 1; LocalEdgeIndexOffset < 3; ++LocalEdgeIndexOffset)
				{
					const int32 LocalEdgeIndex = (LocalEdgeIndexOffset + EdgeFaceCrossingEdgeLocalIndex) % 3;
					if (IntersectionMap.RemoveAndCopyValue({ FaceToEdges_EdgeFace[LocalEdgeIndex], CurrIntersectionFaceIndex }, NextIntersection))
					{
						OutLocalEdgeIndex = LocalEdgeIndex;
						return true;
					}
				}
				return false;
			}

			static inline bool FindNextIntersectionFace(TMap<TVec2<int32>, FEdgeFaceIntersection>& IntersectionMap, const TVec3<int32>& FaceToEdges_Face, const int32 EdgeFace, FEdgeFaceIntersection& NextIntersection, int32& OutLocalEdgeIndex)
			{
				for (int32 LocalEdgeIndex = 0; LocalEdgeIndex < 3; ++LocalEdgeIndex)
				{
					if (IntersectionMap.RemoveAndCopyValue({ FaceToEdges_Face[LocalEdgeIndex], EdgeFace }, NextIntersection))
					{
						OutLocalEdgeIndex = LocalEdgeIndex;
						return true;
					}
				}
				return false;
			}
		} // namespace __internal

		// Build intersection contour data from edge-face intersection data.
		static void BuildIntersectionContours(const FTriangleMesh& TriangleMesh, const TArray<FEdgeFaceIntersection>& IntersectionArray, 
			TArray<FIntersectionContourPair>& Contours, TArray<TArray<FPBDTriangleMeshCollisions::FBarycentricPoint>>& ContourPoints)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ChaosFPBDTriangleMeshCollisions_BuildIntersectionContours);

			// Put intersections in a Map so we can look them up by Edge, Face
			// Key is (EdgeIndex, FaceIndex)
			TMap<TVec2<int32>, FEdgeFaceIntersection> IntersectionMap;
			IntersectionMap.Reserve(IntersectionArray.Num());
			for (const FEdgeFaceIntersection& Intersection : IntersectionArray)
			{
				IntersectionMap.Add({ Intersection.EdgeIndex, Intersection.FaceIndex }, Intersection);
			}

			const FSegmentMesh& SegmentMesh = TriangleMesh.GetSegmentMesh();
			const TArray<TVec2<int32>>& EdgeToFaces = TriangleMesh.GetEdgeToFaces();
			const TArray<TVec3<int32>>& FaceToEdges = TriangleMesh.GetFaceToEdges();
			const TArray<TVec3<int32>>& Elements = TriangleMesh.GetElements();

			while (!IntersectionMap.IsEmpty())
			{
				// Add a new contour
				FIntersectionContourPair& ContourPair = Contours.AddDefaulted_GetRef();

				// Set up first intersection
				TMap<TVec2<int32>, FEdgeFaceIntersection>::TConstIterator Iter = IntersectionMap.CreateConstIterator();
				const FEdgeFaceIntersection FirstIntersection = IntersectionMap.FindAndRemoveChecked(Iter.Key());

				// Track which intersections are used by which contour mostly so we can build contour minimization contours with these.
				ContourPair.Intersections.Add_GetRef(FirstIntersection).bFlipGradient = false;

				// Arbitrarily set colors to start contour
				constexpr int32 InitialFaceContourIndex = 0;
				constexpr int32 InitialEdgeContourIndex = 1;

				FIntersectionContour* FaceContour = &ContourPair.ColorContours[InitialFaceContourIndex];
				FIntersectionContour* EdgeContour = &ContourPair.ColorContours[InitialEdgeContourIndex];
				int32 FaceContourPointsIndex = ContourPoints.AddDefaulted();
				int32 EdgeContourPointsIndex = ContourPoints.AddDefaulted();
				ContourPair.ContourPointCurves[InitialFaceContourIndex].Add(FaceContourPointsIndex);
				ContourPair.ContourPointCurves[InitialEdgeContourIndex].Add(EdgeContourPointsIndex);

				const int32 FirstEdgeFace = EdgeToFaces[FirstIntersection.EdgeIndex][0];
				check(FirstEdgeFace != -1); // Each Edge should be connected to at least one face

				// We will rely on finding the First FaceSection and EdgeFaceSection as the [0] element in these arrays.
				check(FaceContour->Contour.Num() == 0);
				check(EdgeContour->Contour.Num() == 0);


				// Start main loop to consume intersections to build this contour
				bool bReverseDirection = false; // when not reversing, write to CrossingEdgeLocalIndex[0] first. when reversing, writing to CrossingEdgeLocalIndex[1] first.
				FEdgeFaceIntersection CurrIntersection = FirstIntersection;
				int32 EdgeFace = FirstEdgeFace;
				FIntersectionContourTriangleSection* FaceSection = &FaceContour->Contour.Add_GetRef(FIntersectionContourTriangleSection(FirstIntersection.FaceIndex));
				FIntersectionContourTriangleSection* EdgeFaceSection = &EdgeContour->Contour.Add_GetRef(FIntersectionContourTriangleSection(FirstEdgeFace));

				// Setup first EdgeFaceSection crossing
				EdgeFaceSection->CrossingEdgeLocalIndex[0] = __internal::GetLocalEdgeIndex(FaceToEdges, FirstEdgeFace, FirstIntersection.EdgeIndex);
				while (1)
				{
					const TVec3<int32>& FaceVertices = Elements[CurrIntersection.FaceIndex];
					const TVec3<int32>& EdgeFaceVertices = Elements[EdgeFace];

					ContourPoints[FaceContourPointsIndex].Add({CurrIntersection.FaceCoordinate, FaceVertices});
					ContourPoints[EdgeContourPointsIndex].Add({ {CurrIntersection.EdgeCoordinate, 0.f}, {EdgeFaceVertices[0], EdgeFaceVertices[1], EdgeFaceVertices[1]} });

					// Walk the intersection contour. Next point in contour is either
					// 1) Loop vertex
					// 2) Intersection between another edge on EdgeFace and Face
					//   a) This is FirstIntersection (i.e., we closed the loop)
					//   b) This is a previously untouched intersection
					// 3) Intersection between an edge on Face with EdgeFace
					//   a) This is FirstIntersection (i.e., we closed the loop)
					//   b) This is a previously untouched intersection

					// General note on limitations: this algorithm can fail to correctly reconstruct complex contours with combinations of loop vertices and sliver regions that double back on themselves.
					// These contours will be handled by contour minimization.

					// Check 1)
					const int32 EdgeFaceCrossingEdgeLocalIndex = bReverseDirection ? EdgeFaceSection->CrossingEdgeLocalIndex[1] : EdgeFaceSection->CrossingEdgeLocalIndex[0];
					check(EdgeFaceCrossingEdgeLocalIndex != INDEX_NONE);
					const int32 FaceLoopVertexLocalIndex = __internal::FindLoopVertex(FaceVertices, EdgeFaceVertices[(EdgeFaceCrossingEdgeLocalIndex + 2) % 3]);
					if (FaceLoopVertexLocalIndex != INDEX_NONE)
					{
						FaceSection->LoopVertexLocalIndex = FaceLoopVertexLocalIndex;
						EdgeFaceSection->LoopVertexLocalIndex = (EdgeFaceCrossingEdgeLocalIndex + 2) % 3;
						++ContourPair.LoopVertexCount;

						if (ContourPair.NumContourEnds() == 2)
						{
							// We've found both ends of this contour.
							if (ContourPair.LoopVertexCount == 2)
							{
								ContourPair.ClosedStatus = FIntersectionContourPair::EClosedStatus::LoopClosed;
							}
							break;
						}
						check(ContourPair.NumContourEnds() == 1);
						// We hit one loop vertex. Pick up at the FirstIntersection and move the opposite direction if possible.

						EdgeFace = EdgeToFaces[FirstIntersection.EdgeIndex][1];
						if (EdgeFace == -1)
						{
							// First intersection edge was a boundary, so this contour is done. We're NOT closed.
							++ContourPair.ColorContours[InitialEdgeContourIndex].BoundaryEdgeCount;
							break;
						}

						check(bReverseDirection == false);
						bReverseDirection = true;
						CurrIntersection = FirstIntersection;
						FaceContour = &ContourPair.ColorContours[InitialFaceContourIndex];
						EdgeContour = &ContourPair.ColorContours[InitialEdgeContourIndex];

						// ContourPoints are currently just used for debug drawing. Start new contours for reverse section
						FaceContourPointsIndex = ContourPoints.AddDefaulted();
						EdgeContourPointsIndex = ContourPoints.AddDefaulted();
						ContourPair.ContourPointCurves[InitialFaceContourIndex].Add(FaceContourPointsIndex);
						ContourPair.ContourPointCurves[InitialEdgeContourIndex].Add(EdgeContourPointsIndex);

						FaceSection = &FaceContour->Contour[0];
						EdgeFaceSection = &EdgeContour->Contour.Add_GetRef(FIntersectionContourTriangleSection(EdgeFace));
						EdgeFaceSection->CrossingEdgeLocalIndex[1] = __internal::GetLocalEdgeIndex(FaceToEdges, EdgeFace, CurrIntersection.EdgeIndex);
						continue;
					}

					// Check 2a)
					int32 LocalEdgeIndex;
					if (!bReverseDirection && __internal::IsNextIntersectionEdgeFaceFirstIntersection(FirstIntersection, EdgeFaceCrossingEdgeLocalIndex, FaceToEdges[EdgeFace], CurrIntersection.FaceIndex, LocalEdgeIndex))
					{
						// We can close the contour
						EdgeFaceSection->CrossingEdgeLocalIndex[1] = LocalEdgeIndex;

						FIntersectionContourTriangleSection& FirstFaceSection = ContourPair.ColorContours[InitialFaceContourIndex].Contour[0];

						if (!ensure(FaceContour == &ContourPair.ColorContours[InitialFaceContourIndex]) ||
							!ensure(FaceSection->TriangleIndex == FirstFaceSection.TriangleIndex))
						{
							// Somehow our two contours have crossed. Just bail on this contour.
							break;
						}

						// Merge current FaceSection and FirstFaceSection (if they're not already the same)
						if (FaceSection != &FirstFaceSection)
						{
							check(FaceSection->CrossingEdgeLocalIndex[0] != INDEX_NONE);
							FirstFaceSection.CrossingEdgeLocalIndex[0] = FaceSection->CrossingEdgeLocalIndex[0];
							check(FaceSection == &FaceContour->Contour.Last());
							FaceContour->Contour.RemoveAt(FaceContour->Contour.Num() - 1, EAllowShrinking::No);
						}

						ContourPair.ClosedStatus = FIntersectionContourPair::EClosedStatus::SimpleClosed;

						// Repeat first point in contour points for ease of drawing closed loop
						ContourPoints[FaceContourPointsIndex].Add(FPBDTriangleMeshCollisions::FBarycentricPoint(ContourPoints[FaceContourPointsIndex][0]));
						ContourPoints[EdgeContourPointsIndex].Add(FPBDTriangleMeshCollisions::FBarycentricPoint(ContourPoints[EdgeContourPointsIndex][0]));
						break;
					}

					// Check 3a)
					if (!bReverseDirection && __internal::IsNextIntersectionFaceFirstIntersection(FirstIntersection, FaceToEdges[CurrIntersection.FaceIndex], EdgeFace, LocalEdgeIndex))
					{
						// We can close the contour
						FaceSection->CrossingEdgeLocalIndex[1] = LocalEdgeIndex;

						// Merge current EdgeFaceSection with FirstFaceSection
						FIntersectionContourTriangleSection& FirstFaceSection = ContourPair.ColorContours[InitialFaceContourIndex].Contour[0];
						if (!ensure(EdgeContour == &ContourPair.ColorContours[InitialFaceContourIndex]) ||
							!ensure(EdgeFaceSection->TriangleIndex == FirstFaceSection.TriangleIndex))
						{
							// Somehow our two contours have crossed. Just bail on this contour.
							break;
						}

						check(EdgeFaceSection->CrossingEdgeLocalIndex[0] != INDEX_NONE);
						FirstFaceSection.CrossingEdgeLocalIndex[0] = EdgeFaceSection->CrossingEdgeLocalIndex[0];

						check(EdgeFaceSection == &EdgeContour->Contour.Last());
						EdgeContour->Contour.RemoveAt(EdgeContour->Contour.Num() - 1, EAllowShrinking::No);

						ContourPair.ClosedStatus = FIntersectionContourPair::EClosedStatus::SimpleClosed;

						// Repeat first point in contour points for ease of drawing closed loop
						ContourPoints[FaceContourPointsIndex].Add(FPBDTriangleMeshCollisions::FBarycentricPoint(ContourPoints[FaceContourPointsIndex][0]));
						ContourPoints[EdgeContourPointsIndex].Add(FPBDTriangleMeshCollisions::FBarycentricPoint(ContourPoints[EdgeContourPointsIndex][0]));
						break;
					}

					// Check 2b)
					FEdgeFaceIntersection NextIntersection;
					if (__internal::FindNextIntersectionEdgeFace(IntersectionMap, EdgeFaceCrossingEdgeLocalIndex, FaceToEdges[EdgeFace], CurrIntersection.FaceIndex, NextIntersection, LocalEdgeIndex))
					{
						// We found the second crossing for EdgeFaceSection. Mark it. EdgeFaceSection is now complete!
						int32& SecondCrossingEdgeLocalIndex = bReverseDirection ? EdgeFaceSection->CrossingEdgeLocalIndex[0] : EdgeFaceSection->CrossingEdgeLocalIndex[1];
						check(SecondCrossingEdgeLocalIndex == INDEX_NONE);
						SecondCrossingEdgeLocalIndex = LocalEdgeIndex;

						// Add this intersection to the list of intersections consumed by this contour
						const bool bFlipGradient = EdgeContour == &ContourPair.ColorContours[InitialFaceContourIndex]; // Flip gradient if the edge of this intersection belongs to the contour that started with the face of the first intersection
						ContourPair.Intersections.Add_GetRef(NextIntersection).bFlipGradient = bFlipGradient;


						// Add new FaceEdge section by choosing the face edge that doesn't correspond with our current faceEdge (if it exists)

						const int32 NextEdgeFace = EdgeToFaces[NextIntersection.EdgeIndex][0] == EdgeFace ? EdgeToFaces[NextIntersection.EdgeIndex][1] : EdgeToFaces[NextIntersection.EdgeIndex][0];
						if (NextEdgeFace == -1)
						{
							// We've hit a boundary.
							++EdgeContour->BoundaryEdgeCount;

							if (ContourPair.NumContourEnds() == 2)
							{
								// We've hit both ends of this contour.
								if (EdgeContour->BoundaryEdgeCount == 2)
								{
									// This contour is "closed" by hitting boundaries on both sides
									ContourPair.ClosedStatus = FIntersectionContourPair::EClosedStatus::BoundaryClosed;
								}
								break;
							}

							check(ContourPair.NumContourEnds() == 1);

							// Pick up at the FirstIntersection and move the opposite direction if possible.
							EdgeFace = EdgeToFaces[FirstIntersection.EdgeIndex][1];
							if (EdgeFace == -1)
							{
								// First intersection edge was a boundary, so this contour is done.
								++ContourPair.ColorContours[InitialEdgeContourIndex].BoundaryEdgeCount;
								if (EdgeContour->BoundaryEdgeCount == 2)
								{
									// This contour is "closed" by hitting boundaries on both sides
									ContourPair.ClosedStatus = FIntersectionContourPair::EClosedStatus::BoundaryClosed;
								}
								break;
							}

							check(bReverseDirection == false);
							bReverseDirection = true;
							CurrIntersection = FirstIntersection;
							FaceContour = &ContourPair.ColorContours[InitialFaceContourIndex];
							EdgeContour = &ContourPair.ColorContours[InitialEdgeContourIndex];

							// ContourPoints are currently just used for debug drawing. Start new contours for reverse section
							FaceContourPointsIndex = ContourPoints.AddDefaulted();
							EdgeContourPointsIndex = ContourPoints.AddDefaulted();
							ContourPair.ContourPointCurves[InitialFaceContourIndex].Add(FaceContourPointsIndex);
							ContourPair.ContourPointCurves[InitialEdgeContourIndex].Add(EdgeContourPointsIndex);

							FaceSection = &ContourPair.ColorContours[InitialFaceContourIndex].Contour[0];
							EdgeFaceSection = &EdgeContour->Contour.Add_GetRef(FIntersectionContourTriangleSection(EdgeFace));
							EdgeFaceSection->CrossingEdgeLocalIndex[1] = __internal::GetLocalEdgeIndex(FaceToEdges, EdgeFace, CurrIntersection.EdgeIndex);

							continue;
						}

						// Setup for next iteration of loop
						// No change in which contour in the contour pair represents the "face" vs "faceEdge" contour (face section hasn't changed)
						CurrIntersection = NextIntersection;
						EdgeFace = NextEdgeFace;

						EdgeFaceSection = &EdgeContour->Contour.Add_GetRef(FIntersectionContourTriangleSection(EdgeFace));
						int32& FirstCrossingLocalIndex = bReverseDirection ? EdgeFaceSection->CrossingEdgeLocalIndex[1] : EdgeFaceSection->CrossingEdgeLocalIndex[0];
						FirstCrossingLocalIndex = __internal::GetLocalEdgeIndex(FaceToEdges, EdgeFace, CurrIntersection.EdgeIndex);
						continue;
					}

					// Check 3b)
					if (__internal::FindNextIntersectionFace(IntersectionMap, FaceToEdges[CurrIntersection.FaceIndex], EdgeFace, NextIntersection, LocalEdgeIndex))
					{
						// We found the second crossing for FaceSection. Mark it.
						int32& SecondCrossingEdgeLocalIndex = bReverseDirection ? FaceSection->CrossingEdgeLocalIndex[0] : FaceSection->CrossingEdgeLocalIndex[1];
						check(SecondCrossingEdgeLocalIndex == INDEX_NONE);
						SecondCrossingEdgeLocalIndex = LocalEdgeIndex;

						// Add this intersection to the list of intersections consumed by this contour
						const bool bFlipGradient = EdgeContour == &ContourPair.ColorContours[InitialEdgeContourIndex]; // Flip gradient if the edge of this intersection belongs to the contour that started with the face of the first intersection
						ContourPair.Intersections.Add_GetRef(NextIntersection).bFlipGradient = bFlipGradient;

						// Add new FaceEdge section by choosing the face edge that doesn't correspond with our current faceEdge (if it exists)
						const int32 NextEdgeFace = EdgeToFaces[NextIntersection.EdgeIndex][0] == CurrIntersection.FaceIndex ? EdgeToFaces[NextIntersection.EdgeIndex][1] : EdgeToFaces[NextIntersection.EdgeIndex][0];
						if (NextEdgeFace == -1)
						{
							// We've hit a boundary.
							++FaceContour->BoundaryEdgeCount;

							if (ContourPair.NumContourEnds() == 2)
							{
								// We've hit both ends of this contour. 
								if (FaceContour->BoundaryEdgeCount == 2)
								{
									// This contour is "closed" by hitting boundaries on both sides
									ContourPair.ClosedStatus = FIntersectionContourPair::EClosedStatus::BoundaryClosed;
								}
								break;
							}

							check(ContourPair.NumContourEnds() == 1);

							// Pick up at the FirstIntersection and move the opposite direction if possible.
							EdgeFace = EdgeToFaces[FirstIntersection.EdgeIndex][1];
							if (EdgeFace == -1)
							{
								// First intersection edge was a boundary.
								++ContourPair.ColorContours[InitialEdgeContourIndex].BoundaryEdgeCount;
								if (FaceContour->BoundaryEdgeCount == 2)
								{
									// This contour is "closed" by hitting boundaries on both sides
									ContourPair.ClosedStatus = FIntersectionContourPair::EClosedStatus::BoundaryClosed;
								}
								break;
							}

							check(bReverseDirection == false);
							bReverseDirection = true;
							CurrIntersection = FirstIntersection;
							FaceContour = &ContourPair.ColorContours[0];
							EdgeContour = &ContourPair.ColorContours[1];

							// ContourPoints are currently just used for debug drawing. Start new contours for reverse section
							FaceContourPointsIndex = ContourPoints.AddDefaulted();
							EdgeContourPointsIndex = ContourPoints.AddDefaulted();
							ContourPair.ContourPointCurves[InitialFaceContourIndex].Add(FaceContourPointsIndex);
							ContourPair.ContourPointCurves[InitialEdgeContourIndex].Add(EdgeContourPointsIndex);

							FaceSection = &ContourPair.ColorContours[InitialFaceContourIndex].Contour[0];
							EdgeFaceSection = &EdgeContour->Contour.Add_GetRef(FIntersectionContourTriangleSection(EdgeFace));
							EdgeFaceSection->CrossingEdgeLocalIndex[1] = __internal::GetLocalEdgeIndex(FaceToEdges, EdgeFace, CurrIntersection.EdgeIndex);

							continue;
						}

						// Setup for next iteration of loop
						CurrIntersection = NextIntersection;

						// FaceContour and EdgeContour swap
						FIntersectionContour* const TmpContourSwap = EdgeContour;
						EdgeContour = FaceContour;
						FaceContour = TmpContourSwap;
						const int32 TmpContourPointsSwap = EdgeContourPointsIndex;
						EdgeContourPointsIndex = FaceContourPointsIndex;
						FaceContourPointsIndex = TmpContourPointsSwap;

						FaceSection = EdgeFaceSection;

						EdgeFace = NextEdgeFace;
						EdgeFaceSection = &EdgeContour->Contour.Add_GetRef(FIntersectionContourTriangleSection(EdgeFace));
						int32& FirstCrossingLocalIndex = bReverseDirection ? EdgeFaceSection->CrossingEdgeLocalIndex[1] : EdgeFaceSection->CrossingEdgeLocalIndex[0];
						FirstCrossingLocalIndex = __internal::GetLocalEdgeIndex(FaceToEdges, EdgeFace, CurrIntersection.EdgeIndex);
						continue;
					}

					// We failed to find the next intersection in this contour. 
					break;
				}
			};
		}
	} // namespace ContourBuilding

	// methods for Flood-filling contours built by ContourBuilding
	namespace FloodFill
	{
		enum struct EFloodFillRegion : int8
		{
			Untouched = 0,
			Loop,
			Region0,
			Region1,
		};
		static constexpr EFloodFillRegion FloodFillRegions[2] = { EFloodFillRegion::Region0, EFloodFillRegion::Region1 };

		// internal helper methods
		namespace __internal
		{
			static bool SimplifyContourSegments(const FTriangleMesh& TriangleMesh, const TMultiMap<int32 /*TriangleIndex*/, const FIntersectionContourTriangleSection*>& ContourSegments, TMap<int32, FIntersectionContourTriangleSection>& SimplifiedContourSegments, TArray<int32>& TrianglesWithThinFeatures,
				TArray<int32>& CandidateSeedTriangles, TVec2<int32>& LoopVertices)
			{
				// For purposes of vertex-based flood fill, simplify ContourSegments so there is at most one "segment" per triangle.
				// Collapse edge crossings such that two crossing cancel out (i.e., even number = no crossing, odd number = crossing).
				// Keep a list of triangles where there are only even crossing counts--these have thin intersection features and 
				// are handled with the Triangle-based GIAColors (a black vertex is attracted to a white triangle, but the vertices of
				// the white triangle are not attracted to black triangles).
				// This collapsing can effectively generate multiple closed contours (that were originally connected by thin regions).
				// We will need to flood fill them separately. Track the Segments that were formed by pinching off thin regions. These
				// are good seeds for flood filling to ensure all of the separate contours are filled.		
				int32 ContourLoopVertexCount = 0;
				SimplifiedContourSegments.Reset();
				CandidateSeedTriangles.Reset();
				TrianglesWithThinFeatures.Reset();
				LoopVertices = { INDEX_NONE, INDEX_NONE };

				const TArray<TVec3<int32>>& Elements = TriangleMesh.GetElements();

				TArray<int32> UniqueTriangles;
				ContourSegments.GetKeys(UniqueTriangles);

				SimplifiedContourSegments.Reserve(UniqueTriangles.Num());

				for (const int32 TriangleIndex : UniqueTriangles)
				{
					TVec3<int32> CrossingCounts(0);
					TVec3<int32> LoopVertexCounts(0);
					TMultiMap<int32 /*TriangleIndex*/, const FIntersectionContourTriangleSection*>::TConstKeyIterator TriangleIter = ContourSegments.CreateConstKeyIterator(TriangleIndex);
					for (; TriangleIter; ++TriangleIter)
					{
						if (TriangleIter.Value()->LoopVertexLocalIndex != INDEX_NONE)
						{
							++(LoopVertexCounts[TriangleIter.Value()->LoopVertexLocalIndex]);
						}
						if (TriangleIter.Value()->CrossingEdgeLocalIndex[0] != INDEX_NONE)
						{
							++(CrossingCounts[TriangleIter.Value()->CrossingEdgeLocalIndex[0]]);
						}
						if (TriangleIter.Value()->CrossingEdgeLocalIndex[1] != INDEX_NONE)
						{
							++(CrossingCounts[TriangleIter.Value()->CrossingEdgeLocalIndex[1]]);
						}
					}

					const TVec3<int32> SimplifiedCrossingCounts = { CrossingCounts[0] & 1, CrossingCounts[1] & 1, CrossingCounts[2] & 1 };
					const int32 NonZeroCrossingEdgeCount = SimplifiedCrossingCounts[0] + SimplifiedCrossingCounts[1] + SimplifiedCrossingCounts[2];
					if (LoopVertexCounts.Max() > 1)
					{
						// This can occur in pathological loop cases where we have lots of doubling back. These contours aren't actually closed.
						return false;
					}
					const int32 NonZeroLoopVertexCount = LoopVertexCounts[0] + LoopVertexCounts[1] + LoopVertexCounts[2];
					if ((NonZeroLoopVertexCount + NonZeroCrossingEdgeCount) % 2 != 0)
					{
						// Similarly, this can occur with slivers doubling back. These contours aren't actually closed.
						return false;
					}

					if (NonZeroCrossingEdgeCount + NonZeroLoopVertexCount == 0)
					{
						TrianglesWithThinFeatures.Add(TriangleIndex);
						continue;
					}

					if (NonZeroLoopVertexCount > 0)
					{
						// Triangle with loop vertex
						FIntersectionContourTriangleSection SimplifiedSegment(TriangleIndex);
						for (int32 LocalVertexIdx = 0; LocalVertexIdx < 3; ++LocalVertexIdx)
						{
							if (LoopVertexCounts[LocalVertexIdx])
							{
								const int32 LoopVertexGlobalIdx = Elements[TriangleIndex][LocalVertexIdx];
								if (LoopVertices[0] != LoopVertexGlobalIdx && LoopVertices[1] != LoopVertexGlobalIdx)
								{
									check(ContourLoopVertexCount < 2);
									LoopVertices[ContourLoopVertexCount++] = LoopVertexGlobalIdx;
								}
								SimplifiedSegment.LoopVertexLocalIndex = LocalVertexIdx; // It doesn't matter in the Simplified context if there's more than one loop vertex and we only record one because we just care if a loop vertex exists or not.
							}
						}
						SimplifiedContourSegments.Add(TriangleIndex, MoveTemp(SimplifiedSegment));
					}
					else
					{
						// Crossings
						FIntersectionContourTriangleSection SimplifiedSegment(TriangleIndex);
						check(NonZeroCrossingEdgeCount <= 2);
						int32 CrossingEdgeIdx = 0;
						for (int32 LocalEdgeIdx = 0; LocalEdgeIdx < 3; ++LocalEdgeIdx)
						{
							if (SimplifiedCrossingCounts[LocalEdgeIdx])
							{
								SimplifiedSegment.CrossingEdgeLocalIndex[CrossingEdgeIdx++] = LocalEdgeIdx;
							}
						}
						SimplifiedContourSegments.Add(TriangleIndex, MoveTemp(SimplifiedSegment));
						if (CrossingCounts[0] && CrossingCounts[1] && CrossingCounts[2])
						{
							// This used to have crossings on 3 sides, and one was pinched off. This is a candidate seed triangle
							CandidateSeedTriangles.Add(TriangleIndex);
						}
					}
				}
				return true;
			}

			// Returns success
			static bool OneFloodFillStep(const TMap<int32 /*TriangleIndex*/, FIntersectionContourTriangleSection>& ContourSegments, const TArray<TVec3<int32>>& Elements, const TConstArrayView<TArray<int32>>& PointToTriangleMap, TArray<int32>& Queue, int32& RegionSize, TArrayView<EFloodFillRegion>& VertexRegions,
				const EFloodFillRegion RegionColor, const EFloodFillRegion OtherRegionColor)
			{
				const int32 CurrVertex = Queue.Pop(EAllowShrinking::No);
				for (int32 NeighborTri : PointToTriangleMap[CurrVertex])
				{
					if (const FIntersectionContourTriangleSection* TriSection = ContourSegments.Find(NeighborTri))
					{
						const int32 CurrVertexLocalIndex = CurrVertex == Elements[NeighborTri][0] ? 0 : CurrVertex == Elements[NeighborTri][1] ? 1 : CurrVertex == Elements[NeighborTri][2] ? 2 : INDEX_NONE;
						check(CurrVertexLocalIndex != INDEX_NONE);

						if (TriSection->LoopVertexLocalIndex != INDEX_NONE)
						{
							// Triangles with loop vertices don't connect other vertices within the triangle
							// We will mark loop vertices elsewhere
							continue;
						}

						check(TriSection->CrossingEdgeLocalIndex[0] != INDEX_NONE && TriSection->CrossingEdgeLocalIndex[1] != INDEX_NONE);
						check(TriSection->CrossingEdgeLocalIndex[0] != TriSection->CrossingEdgeLocalIndex[1]);

						// Order such that SecondCrossEdge = (FirstCrossEdge + 1)%3
						const bool bSwapEdgeOrder = TriSection->CrossingEdgeLocalIndex[1] != (TriSection->CrossingEdgeLocalIndex[0] + 1) % 3;
						const int32 FirstCrossEdge = bSwapEdgeOrder ? TriSection->CrossingEdgeLocalIndex[1] : TriSection->CrossingEdgeLocalIndex[0];
						// const int32 SecondCrossEdge = bSwapEdgeOrder ? TriSection->CrossingEdgeLocalIndex[0] : TriSection->CrossingEdgeLocalIndex[1];

						if (FirstCrossEdge == CurrVertexLocalIndex)
						{
							// CurrVertex is on the same side as CurrVertex - 1
							const int32 TriVertex = Elements[NeighborTri][(CurrVertexLocalIndex + 2) % 3];
							if (VertexRegions[TriVertex] == EFloodFillRegion::Untouched)
							{
								VertexRegions[TriVertex] = RegionColor;
								++RegionSize;
								Queue.Add(TriVertex);
							}
							else if (VertexRegions[TriVertex] == OtherRegionColor)
							{
								// We don't have a closed contour after all.
								return false;
							}
						}
						else if (FirstCrossEdge == (CurrVertexLocalIndex + 1) % 3)
						{
							// CurrVertex is on the same side as CurrVertex + 1
							const int32 TriVertex = Elements[NeighborTri][(CurrVertexLocalIndex + 1) % 3];
							if (VertexRegions[TriVertex] == EFloodFillRegion::Untouched)
							{
								VertexRegions[TriVertex] = RegionColor;
								++RegionSize;
								Queue.Add(TriVertex);
							}
							else if (VertexRegions[TriVertex] == OtherRegionColor)
							{
								// We don't have a closed contour after all.
								return false;
							}
						}
						// else CurrVertex is not on the same side with any other vertices in this triangle
					}
					else
					{
						// Contour doesn't pass through this tri. Just add all (untouched) points to this Region
						for (int32 VertexLocalIndex = 0; VertexLocalIndex < 3; ++VertexLocalIndex)
						{
							const int32 TriVertex = Elements[NeighborTri][VertexLocalIndex];
							if (VertexRegions[TriVertex] == EFloodFillRegion::Untouched)
							{
								VertexRegions[TriVertex] = RegionColor;
								++RegionSize;
								Queue.Add(TriVertex);
							}
							else if (VertexRegions[TriVertex] == OtherRegionColor)
							{
								// We don't have a closed contour after all.
								return false;
							}
						}
					}
				}
				return true;
			}

			// Flood fill a single contour
			static bool FloodFillContourColor(const FTriangleMesh& TriangleMesh, const int32 NumParticles, const int32 Offset, const TMultiMap<int32 /*TriangleIndex*/, const FIntersectionContourTriangleSection*>& ContourSegments, int32 ContourIndex, bool bIsColorB,
				TArray<FPBDTriangleMeshCollisions::FGIAColor>& VertexGIAColors, TArray<FPBDTriangleMeshCollisions::FGIAColor>& TriangleGIAColors)
			{
				// Simplify Contour 
				TMap<int32, FIntersectionContourTriangleSection> SimplifiedContourSegments;
				TArray<int32> TrianglesWithThinFeatures;
				TArray<int32> CandidateSeedTriangles;
				TVec2<int32> LoopVertices;
				if (!SimplifyContourSegments(TriangleMesh, ContourSegments, SimplifiedContourSegments, TrianglesWithThinFeatures, CandidateSeedTriangles, LoopVertices))
				{
					return false;
				}

				const TArray<TVec3<int32>>& Elements = TriangleMesh.GetElements();
				const TConstArrayView<TArray<int32>>& PointToTrianglemMap = TriangleMesh.GetPointToTriangleMap();

				// Main contour flood fill
				if (SimplifiedContourSegments.Num())
				{
					// This collapsing can effectively generate multiple closed contours (that were originally connected by thin regions).
					// We will need to flood fill them separately.
					if (CandidateSeedTriangles.Num() == 0)
					{
						// If there weren't any candidates found by pinching regions, just start with first non-loop segment
						TMap<int32, FIntersectionContourTriangleSection>::TConstIterator Iter = SimplifiedContourSegments.CreateConstIterator();
						for (; Iter; ++Iter)
						{
							const FIntersectionContourTriangleSection& FirstSection = Iter.Value();
							if (FirstSection.LoopVertexLocalIndex != INDEX_NONE)
							{
								continue;
							}

							const int32 FirstVertex0 = FirstSection.CrossingEdgeLocalIndex[0];
							const int32 FirstVertex1 = (FirstSection.CrossingEdgeLocalIndex[0] + 1) % 3;
							if (Elements[Iter.Key()][FirstVertex0] == LoopVertices[0] || Elements[Iter.Key()][FirstVertex0] == LoopVertices[1] ||
								Elements[Iter.Key()][FirstVertex1] == LoopVertices[0] || Elements[Iter.Key()][FirstVertex1] == LoopVertices[1])
							{
								continue;
							}
							CandidateSeedTriangles.Add(Iter.Key());
							break;
						}
					}

					for (const int32 SeedTriangle : CandidateSeedTriangles)
					{
						// Initialize queue for flood fill with first Section in Contour
						const FIntersectionContourTriangleSection& FirstSection = SimplifiedContourSegments.FindChecked(SeedTriangle);
						check(FirstSection.CrossingEdgeLocalIndex[0] != INDEX_NONE);
						const int32 FirstVertex0 = FirstSection.CrossingEdgeLocalIndex[0];
						const int32 FirstVertex1 = (FirstSection.CrossingEdgeLocalIndex[0] + 1) % 3;

						// Check if this Triangle already has been colored on either side.
						if (VertexGIAColors[Elements[FirstSection.TriangleIndex][FirstVertex0] - Offset].HasContourColorSet(ContourIndex)
							|| VertexGIAColors[Elements[FirstSection.TriangleIndex][FirstVertex1] - Offset].HasContourColorSet(ContourIndex))
						{
							continue;
						}

						// Flood fill both sides of contour. Stop when one side is complete -- we want the smaller region.
						// For now, assume triangle areas are similar, and just calculate "smaller" based on number of vertices

						TArray<int32 /*VertexIndex*/> Queue[2];
						int32 RegionSizes[2] = { 0, 0 };
						TArray<EFloodFillRegion> VertexRegionsNoOffset;
						static_assert((int8)EFloodFillRegion::Untouched == 0);
						VertexRegionsNoOffset.SetNumZeroed(NumParticles);

						TArrayView<EFloodFillRegion> VertexRegions(VertexRegionsNoOffset.GetData() - Offset, Offset + NumParticles);

						Queue[0].Add(Elements[FirstSection.TriangleIndex][FirstVertex0]);
						Queue[1].Add(Elements[FirstSection.TriangleIndex][FirstVertex1]);

						VertexRegions[Elements[FirstSection.TriangleIndex][FirstVertex0]] = FloodFillRegions[0];
						VertexRegions[Elements[FirstSection.TriangleIndex][FirstVertex1]] = FloodFillRegions[1];
						++RegionSizes[0];
						++RegionSizes[1];

						// Set Loop vertices so we don't cross them when flooding
						if (LoopVertices[0] != INDEX_NONE)
						{
							VertexRegions[LoopVertices[0]] = EFloodFillRegion::Loop;
						}
						if (LoopVertices[1] != INDEX_NONE)
						{
							VertexRegions[LoopVertices[1]] = EFloodFillRegion::Loop;
						}

						while (Queue[0].Num() > 0 && Queue[1].Num() > 0)
						{
							for (int32 RegionIndex = 0; RegionIndex < 2; ++RegionIndex)
							{
								if (!OneFloodFillStep(SimplifiedContourSegments, Elements, PointToTrianglemMap, Queue[RegionIndex], RegionSizes[RegionIndex], VertexRegions, FloodFillRegions[RegionIndex], FloodFillRegions[RegionIndex ^ 1]))
								{
									return false;
								}
							}
						}

						const int32 CompletedQueueIndex = Queue[0].Num() == 0 ? 0 : 1;
						if (RegionSizes[CompletedQueueIndex] >= RegionSizes[CompletedQueueIndex ^ 1])
						{
							// Run FloodFill on incomplete queue until complete/larger than completed queue
							while (Queue[CompletedQueueIndex ^ 1].Num() > 0 && (RegionSizes[CompletedQueueIndex] >= RegionSizes[CompletedQueueIndex ^ 1]))
							{
								if (!OneFloodFillStep(SimplifiedContourSegments, Elements, PointToTrianglemMap, Queue[CompletedQueueIndex ^ 1], RegionSizes[CompletedQueueIndex ^ 1], VertexRegions, FloodFillRegions[CompletedQueueIndex ^ 1], FloodFillRegions[CompletedQueueIndex]))
								{
									return false;
								}
							}
						}

						if (RegionSizes[0] == RegionSizes[1])
						{
							// Both regions are the same size. Can't determine inside.
							return false;
						}

						// Mark VertexGIAColors for smaller region
						const EFloodFillRegion SmallerRegionColor = RegionSizes[0] < RegionSizes[1] ? FloodFillRegions[0] : FloodFillRegions[1];
						for (int ParticleIndexNoOffset = 0; ParticleIndexNoOffset < NumParticles; ++ParticleIndexNoOffset)
						{
							if (VertexRegionsNoOffset[ParticleIndexNoOffset] == SmallerRegionColor)
							{
								VertexGIAColors[ParticleIndexNoOffset].SetContourColor(ContourIndex, bIsColorB);
							}
						}
					}
				}

				// Loop vertex handling
				if (ContourIndex == FPBDTriangleMeshCollisions::FGIAColor::LoopContourIndex)
				{
					if (LoopVertices[0] != INDEX_NONE)
					{
						VertexGIAColors[LoopVertices[0] - Offset].SetContourColor(ContourIndex, bIsColorB);
					}
					if (LoopVertices[1] != INDEX_NONE)
					{
						VertexGIAColors[LoopVertices[1] - Offset].SetContourColor(ContourIndex, bIsColorB);
					}
				}

				// Thin region triangles
				for (int32 TriangleIndex : TrianglesWithThinFeatures)
				{
					TriangleGIAColors[TriangleIndex].SetContourColor(ContourIndex, bIsColorB);
				}
				return true;
			}
		} // namespace __internal

		// Flood fill contours built by ContourBuilding to generate VertexGIAColors. Identify triangles with thin intersections (TriangleGIAColors)
		static void FloodFillContours(const FTriangleMesh& TriangleMesh, const int32 NumParticles, const int32 Offset, TArray<FIntersectionContourPair>& Contours, TArray<FPBDTriangleMeshCollisions::FGIAColor>& VertexGIAColors, TArray<FPBDTriangleMeshCollisions::FGIAColor>& TriangleGIAColors)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ChaosFPBDTriangleMeshCollisions_FloodFillContours);
			const TArray<TVec3<int32>>& Elements = TriangleMesh.GetElements();
			const TConstArrayView<TArray<int32>>& PointToTrianglemMap = TriangleMesh.GetPointToTriangleMap();

			// If we used a critsect/made writing to the GIAColors threadsafe with atomics, we parallelize by contour. Not sure if it's worth the overhead though.

			int32 CurrentContourIndex = 1; // ContourIndex 0 is reserved for loop 
			for (const FIntersectionContourPair& Contour : Contours)
			{
				if (Contour.ClosedStatus == FIntersectionContourPair::EClosedStatus::Open)
				{
					continue;
				}

				TMultiMap<int32 /*TriangleIndex*/, const FIntersectionContourTriangleSection*> ContourSegments;
				
				const bool bTwoSimpleContours = Contour.ClosedStatus == FIntersectionContourPair::EClosedStatus::SimpleClosed;

				int32 ContourIndex = INDEX_NONE;
				bool bIsColorB = false;
				switch (Contour.ClosedStatus)
				{
				case FIntersectionContourPair::EClosedStatus::SimpleClosed:
				{
					ContourIndex = (((CurrentContourIndex++) - 1) % 31) + 1;
					bIsColorB = false;
					ContourSegments.Reserve(FMath::Max(Contour.ColorContours[0].Contour.Num(), Contour.ColorContours[1].Contour.Num()));

					for (const FIntersectionContourTriangleSection& Section : Contour.ColorContours[0].Contour)
					{
						ContourSegments.Emplace(Section.TriangleIndex, &Section);
					}
				}break;
				case FIntersectionContourPair::EClosedStatus::LoopClosed:
				{
					ContourIndex = FPBDTriangleMeshCollisions::FGIAColor::LoopContourIndex;
					bIsColorB = true;
					ContourSegments.Reserve(Contour.ColorContours[0].Contour.Num() + Contour.ColorContours[1].Contour.Num());
					for (const FIntersectionContourTriangleSection& Section : Contour.ColorContours[0].Contour)
					{
						ContourSegments.Emplace(Section.TriangleIndex, &Section);
					}
					for (const FIntersectionContourTriangleSection& Section : Contour.ColorContours[1].Contour)
					{
						ContourSegments.Emplace(Section.TriangleIndex, &Section);
					}
				}break;
				case FIntersectionContourPair::EClosedStatus::BoundaryClosed:
				{
					ContourIndex = FPBDTriangleMeshCollisions::FGIAColor::BoundaryContourIndex;
					bIsColorB = false;
					if (Contour.ColorContours[0].BoundaryEdgeCount == 2)
					{
						ContourSegments.Reserve(Contour.ColorContours[0].Contour.Num());

						for (const FIntersectionContourTriangleSection& Section : Contour.ColorContours[0].Contour)
						{
							ContourSegments.Emplace(Section.TriangleIndex, &Section);
						}
					}
					else
					{
						check(Contour.ColorContours[1].BoundaryEdgeCount == 2);
						ContourSegments.Reserve(Contour.ColorContours[1].Contour.Num());

						for (const FIntersectionContourTriangleSection& Section : Contour.ColorContours[1].Contour)
						{
							ContourSegments.Emplace(Section.TriangleIndex, &Section);
						}
					}
				}break;
				default:
					checkNoEntry();
				}

				check(ContourIndex != INDEX_NONE);

				if (!__internal::FloodFillContourColor(TriangleMesh, NumParticles, Offset, ContourSegments, ContourIndex, bIsColorB, VertexGIAColors, TriangleGIAColors))
				{
					const_cast<FIntersectionContourPair&>(Contour).ClosedStatus = FIntersectionContourPair::EClosedStatus::Open;
					continue;
				}

				if (bTwoSimpleContours)
				{
					ContourSegments.Reset();

					for (const FIntersectionContourTriangleSection& Section : Contour.ColorContours[1].Contour)
					{
						ContourSegments.Emplace(Section.TriangleIndex, &Section);
					}

					if (!__internal::FloodFillContourColor(TriangleMesh, NumParticles, Offset, ContourSegments, ContourIndex, true, VertexGIAColors, TriangleGIAColors))
					{
						const_cast<FIntersectionContourPair&>(Contour).ClosedStatus = FIntersectionContourPair::EClosedStatus::Open;
						continue;
					}
				}
			}
		}

		// Once FloodFill has completed, we can mark the IntersectionContourTypes for debug drawing ContourPoints.
		static void AssignContourPointTypes(const TArray<FIntersectionContourPair>& Contours, const TArray<TArray<FPBDTriangleMeshCollisions::FBarycentricPoint>>& ContourPoints, TArray<FPBDTriangleMeshCollisions::FContourType>& IntersectionContourTypes)
		{
			IntersectionContourTypes.Reset();
			IntersectionContourTypes.SetNumZeroed(ContourPoints.Num());
			for (const FIntersectionContourPair& ContourPair : Contours)
			{
				switch (ContourPair.ClosedStatus)
				{
				case FIntersectionContourPair::EClosedStatus::SimpleClosed:
				{
					for (const int32 PointIndex : ContourPair.ContourPointCurves[0])
					{
						IntersectionContourTypes[PointIndex] = FPBDTriangleMeshCollisions::FContourType::Contour0;
					}
					for (const int32 PointIndex : ContourPair.ContourPointCurves[1])
					{
						IntersectionContourTypes[PointIndex] = FPBDTriangleMeshCollisions::FContourType::Contour1;
					}
				}break;
				case FIntersectionContourPair::EClosedStatus::LoopClosed:
				{
					for (const int32 PointIndex : ContourPair.ContourPointCurves[0])
					{
						IntersectionContourTypes[PointIndex] = FPBDTriangleMeshCollisions::FContourType::Loop;
					}
					for (const int32 PointIndex : ContourPair.ContourPointCurves[1])
					{
						IntersectionContourTypes[PointIndex] = FPBDTriangleMeshCollisions::FContourType::Loop;
					}
				}break;
				case FIntersectionContourPair::EClosedStatus::BoundaryClosed:
				{
					for (int32 ContourIndex = 0; ContourIndex < 2; ++ContourIndex)
					{
						const FPBDTriangleMeshCollisions::FContourType ContourType =
							ContourPair.ColorContours[ContourIndex].BoundaryEdgeCount == 2 ?
							FPBDTriangleMeshCollisions::FContourType::BoundaryClosed
							: FPBDTriangleMeshCollisions::FContourType::BoundaryOpen;

						for (const int32 PointIndex : ContourPair.ContourPointCurves[ContourIndex])
						{
							IntersectionContourTypes[PointIndex] = ContourType;;
						}
					}
				}
				break;
				default:
					break;
				}
			}
		}
	} // namespace FloodFill
} // namespace GIA

// Calculating gradient direction to minimize contour length.
namespace ContourMinimization
{
	template<typename SolverParticlesOrRange>
	static void BuildLocalContourMinimizationIntersection(const FEdgeFaceIntersection& EdgeFaceIntersection, const FTriangleMesh& TriangleMesh, const FSegmentMesh& SegmentMesh, const SolverParticlesOrRange& Particles, FPBDTriangleMeshCollisions::FContourMinimizationIntersection& ContourIntersection)
	{
		ContourIntersection.EdgeVertices = SegmentMesh.GetElements()[EdgeFaceIntersection.EdgeIndex];
		ContourIntersection.FaceVertices = TriangleMesh.GetElements()[EdgeFaceIntersection.FaceIndex];
		ContourIntersection.LocalGradientVector = FSolverVec3(0.f);

		// Calculate local gradient
		const TArray<TVec2<int32>>& EdgeToFaces = TriangleMesh.GetEdgeToFaces();
		const TArray<TVec3<int32>>& Elements = TriangleMesh.GetElements();

		const FSolverVec3& EdgePosition0 = Particles.GetX(ContourIntersection.EdgeVertices[0]);
		const FSolverVec3& EdgePosition1 = Particles.GetX(ContourIntersection.EdgeVertices[1]);
		const FSolverVec3 EdgeDir = EdgePosition1 - EdgePosition0; // Unnormalized
		const FSolverReal EdgeDirDotNormal = FSolverVec3::DotProduct(EdgeDir, EdgeFaceIntersection.FaceNormal);
		if (FMath::Abs(EdgeDirDotNormal) >= UE_SMALL_NUMBER)
		{
			const FSolverVec3 NOverEDotN = EdgeFaceIntersection.FaceNormal * ((FSolverReal)1.f / EdgeDirDotNormal);

			for (int32 LocalFaceIndex = 0; LocalFaceIndex < 2; ++LocalFaceIndex)
			{
				const int32 EdgeFace_i = EdgeToFaces[EdgeFaceIntersection.EdgeIndex][LocalFaceIndex];
				if (EdgeFace_i == -1)
				{
					continue;
				}

				const FSolverVec3& P0 = Particles.GetX(Elements[EdgeFace_i][0]);
				const FSolverVec3& P1 = Particles.GetX(Elements[EdgeFace_i][1]);
				const FSolverVec3& P2 = Particles.GetX(Elements[EdgeFace_i][2]);

				// B_i
				const TTriangle<FSolverReal> Triangle_i(P0, P1, P2);

				// N_i
				const FSolverVec3 Normal_i = Triangle_i.GetNormal();

				// R_i
				FSolverVec3 IntersectionVector_i = FSolverVec3::CrossProduct(EdgeFaceIntersection.FaceNormal, Normal_i);
				IntersectionVector_i.Normalize();
				// This should point inward from the Edge. Flip if necessary.

				// SideDirection = Normal_i ^ (EdgeDir) // NOTE: Need to calculate EdgeDir based on vertex order within the triangle, not what's in the SegmentMesh to ensure SideDirection points INTO from triangle.
				// If Dot(IntersectionVector_i, SideDirection) < 0, need to flip IntersectionVector_i
				// Dot(IntersectionVector_i, SideDirection) = Dot((CollisionPoint.Normal ^ Normal_i), (Normal_i ^ EdgeDir))
				// Using triple product rules, this is
				// Dot(CollisionPoint.Normal, Normal_i) * Dot(Normal_i, EdgeDir) - Dot(CollisionPoint.Normal, EdgeDir) * Dot(Normal_i, Normal_i)
				const int32 EdgePointLocalIndex0 = ContourIntersection.EdgeVertices[0] == Elements[EdgeFace_i][0] ? 0 : ContourIntersection.EdgeVertices[0] == Elements[EdgeFace_i][1] ? 1 : ContourIntersection.EdgeVertices[0] == Elements[EdgeFace_i][2] ? 2 : INDEX_NONE;
				check(EdgePointLocalIndex0 != INDEX_NONE); // Otherwise the EdgeToFaces is messed up because this edge doesn't below to this face
				check(ContourIntersection.EdgeVertices[1] == Elements[EdgeFace_i][(EdgePointLocalIndex0 + 1) % 3] || ContourIntersection.EdgeVertices[1] == Elements[EdgeFace_i][(EdgePointLocalIndex0 + 2) % 3]);
				const bool bEdgePointsFlipped = ContourIntersection.EdgeVertices[1] != Elements[EdgeFace_i][(EdgePointLocalIndex0 + 1) % 3];
				const FSolverReal TestFlipRValue = (FSolverVec3::DotProduct(EdgeFaceIntersection.FaceNormal, Normal_i) * FSolverVec3::DotProduct(Normal_i, EdgeDir) - FSolverVec3::DotProduct(EdgeFaceIntersection.FaceNormal, EdgeDir)) * (bEdgePointsFlipped ? (FSolverReal)-1.f : (FSolverReal)1.f);
				if (TestFlipRValue < 0)
				{
					IntersectionVector_i *= (FSolverReal)-1.f;
				}

				ContourIntersection.LocalGradientVector += IntersectionVector_i - (FSolverReal)2.f * FSolverVec3::DotProduct(EdgeDir, IntersectionVector_i) * NOverEDotN;
			}
		}

		ContourIntersection.GlobalGradientVector = ContourIntersection.LocalGradientVector;
	}

	template<typename SolverParticlesOrRange>
	static void BuildLocalContourMinimizationIntersections(const FTriangleMesh& TriangleMesh, const SolverParticlesOrRange& Particles, const TArray<FEdgeFaceIntersection>& Intersections, TArray<FPBDTriangleMeshCollisions::FContourMinimizationIntersection>& ContourMinimizationIntersections)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ChaosFPBDTriangleMeshCollisions_BuildLocalContourMinimizationIntersections);
		const FSegmentMesh& SegmentMesh = TriangleMesh.GetSegmentMesh();

		ContourMinimizationIntersections.SetNum(Intersections.Num());
		PhysicsParallelFor(Intersections.Num(),
			[&TriangleMesh, &SegmentMesh, &Particles, &Intersections, &ContourMinimizationIntersections](int32 IntersectionIndex)
			{
				const FEdgeFaceIntersection& EdgeFaceIntersection = Intersections[IntersectionIndex];
				FPBDTriangleMeshCollisions::FContourMinimizationIntersection& ContourIntersection = ContourMinimizationIntersections[IntersectionIndex];
				BuildLocalContourMinimizationIntersection(EdgeFaceIntersection, TriangleMesh, SegmentMesh, Particles, ContourIntersection);
			}
		);
	}

	template<typename SolverParticlesOrRange>
	static void BuildGlobalContourMinimizationIntersections(const FTriangleMesh& TriangleMesh, const SolverParticlesOrRange& Particles, const TArray<GIA::FIntersectionContourPair>& IntersectionContours, TArray<FPBDTriangleMeshCollisions::FContourMinimizationIntersection>& ContourMinimizationIntersections)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ChaosFPBDTriangleMeshCollisions_BuildGlobalContourMinimizationIntersections);
		const FSegmentMesh& SegmentMesh = TriangleMesh.GetSegmentMesh();
		// Only do contour minimization for non-closed contours. Closed contours will be handled via GIA

		// Determine total number of non-closed contours and number of segments
		struct FContourRange
		{
			int32 ContourPairIndex;
			int32 GlobalIndexStart;
		};

		TArray<FContourRange> ContourRanges;
		ContourRanges.Reset(IntersectionContours.Num());
		int32 IndexStart = 0;
		for (int32 ContourPairIndex = 0; ContourPairIndex < IntersectionContours.Num(); ++ContourPairIndex)
		{
			if (IntersectionContours[ContourPairIndex].ClosedStatus == GIA::FIntersectionContourPair::EClosedStatus::SimpleClosed)
			{
				continue;
			}
			ContourRanges.Add({ ContourPairIndex, IndexStart });
			IndexStart += IntersectionContours[ContourPairIndex].Intersections.Num();
		}

		if (ContourRanges.Num() == 0)
		{
			return;
		}

		ContourMinimizationIntersections.SetNum(IndexStart);

		// For now, parallelize by contour. Might want to balance by contour length?
		PhysicsParallelFor(ContourRanges.Num(),
			[&TriangleMesh, &SegmentMesh, &Particles, &ContourRanges, &IntersectionContours, &ContourMinimizationIntersections](int32 ContourRangeIndex)
			{ 
				const TArray<FEdgeFaceIntersection>& Intersections = IntersectionContours[ContourRanges[ContourRangeIndex].ContourPairIndex].Intersections;
				const int32 GlobalIndexStart = ContourRanges[ContourRangeIndex].GlobalIndexStart;
				FSolverVec3 GlobalGradient(0.f);
				for (int32 SegmentIndex = 0; SegmentIndex < Intersections.Num(); ++SegmentIndex)
				{
					const FEdgeFaceIntersection& EdgeFaceIntersection = Intersections[SegmentIndex];
					FPBDTriangleMeshCollisions::FContourMinimizationIntersection& ContourIntersection = ContourMinimizationIntersections[GlobalIndexStart + SegmentIndex];
					BuildLocalContourMinimizationIntersection(EdgeFaceIntersection, TriangleMesh, SegmentMesh, Particles, ContourIntersection);
					GlobalGradient += ContourIntersection.LocalGradientVector * (EdgeFaceIntersection.bFlipGradient ? -1.f : 1.f);
				}

				for (int32 SegmentIndex = 0; SegmentIndex < Intersections.Num(); ++SegmentIndex)
				{
					const FEdgeFaceIntersection& EdgeFaceIntersection = Intersections[SegmentIndex];
					FPBDTriangleMeshCollisions::FContourMinimizationIntersection& ContourIntersection = ContourMinimizationIntersections[GlobalIndexStart + SegmentIndex];
					ContourIntersection.GlobalGradientVector = GlobalGradient * (EdgeFaceIntersection.bFlipGradient ? -1.f : 1.f);
				}
			}
		);		
	}
} // namespace ContourMinimization

struct FPBDTriangleMeshCollisions::FScratchBuffers
{
	TArray<FEdgeFaceIntersection> EdgeFaceIntersections;
	TArray<GIA::FIntersectionContourPair> IntersectionContours;

	void Reset()
	{
		EdgeFaceIntersections.Reset();
		IntersectionContours.Reset();
	}
};

template<typename SolverParticlesOrRange>
void FPBDTriangleMeshCollisions::FTriangleSubMesh::Init(const SolverParticlesOrRange& Particles, const TSet<int32>& InDisabledFaces, bool bCollideAgainstAllKinematicVertices, const TSet<int32>& InEnabledKinematicFaces, const bool bOnlyCollideKinematics)
{
	FullMeshToSubMeshIndices.Reset();
	DynamicSubMeshToFullMeshIndices.Reset();
	KinematicColliderSubMeshToFullMeshIndices.Reset();
	DynamicVertices.Reset();

	TArray<TVec3<int32>> DynamicMeshElements;
	TArray<TVec3<int32>> KinematicMeshElements;
	if (!bOnlyCollideKinematics)
	{
		DynamicMeshElements.Reserve(FullMesh.GetNumElements());
	}
	KinematicMeshElements.Reserve(InEnabledKinematicFaces.Num());

	FullMeshToSubMeshIndices.SetNumZeroed(FullMesh.GetNumElements());
	DynamicSubMeshToFullMeshIndices.Reserve(FullMesh.GetNumElements());
	KinematicColliderSubMeshToFullMeshIndices.Reserve(InEnabledKinematicFaces.Num());

	const TArray<TVec3<int32>>& FullElements = FullMesh.GetElements();
	for (int32 FullElementIndex = 0; FullElementIndex < FullMesh.GetNumElements(); ++FullElementIndex)
	{
		if (InDisabledFaces.Contains(FullElementIndex))
		{
			FullMeshToSubMeshIndices[FullElementIndex].SubMeshType = ESubMeshType::Invalid;
			continue;
		}
		const bool bIsKinematic =
			Particles.InvM(FullElements[FullElementIndex][0]) == (FSolverReal)0.f &&
			Particles.InvM(FullElements[FullElementIndex][1]) == (FSolverReal)0.f &&
			Particles.InvM(FullElements[FullElementIndex][2]) == (FSolverReal)0.f;

		if (bIsKinematic)
		{
			if (bCollideAgainstAllKinematicVertices || InEnabledKinematicFaces.Contains(FullElementIndex))
			{
				const int32 SubMeshIndex = KinematicMeshElements.Add(FullElements[FullElementIndex]);
				KinematicColliderSubMeshToFullMeshIndices.Add(FullElementIndex);
				FullMeshToSubMeshIndices[FullElementIndex].SubMeshIndex = SubMeshIndex;
				FullMeshToSubMeshIndices[FullElementIndex].SubMeshType = ESubMeshType::Kinematic;
			}
			else
			{
				FullMeshToSubMeshIndices[FullElementIndex].SubMeshType = ESubMeshType::Invalid;
			}
		}
		else if (!bOnlyCollideKinematics)
		{
			const int32 SubMeshIndex = DynamicMeshElements.Add(FullElements[FullElementIndex]);
			DynamicSubMeshToFullMeshIndices.Add(FullElementIndex);
			FullMeshToSubMeshIndices[FullElementIndex].SubMeshIndex = SubMeshIndex;
			FullMeshToSubMeshIndices[FullElementIndex].SubMeshType = ESubMeshType::Dynamic;
		}
	}

	// Use same Vertex range for submeshes.
	const TVec2<int32> VertexRange = FullMesh.GetVertexRange();
	constexpr bool bCullDegenerateFalse = false;

	DynamicSubMesh.Init(MoveTemp(DynamicMeshElements), VertexRange[0], VertexRange[1], bCullDegenerateFalse);
	KinematicColliderSubMesh.Init(MoveTemp(KinematicMeshElements), VertexRange[0], VertexRange[1], bCullDegenerateFalse);

	check(DynamicSubMesh.GetNumElements() == DynamicSubMeshToFullMeshIndices.Num());
	check(KinematicColliderSubMesh.GetNumElements() == KinematicColliderSubMeshToFullMeshIndices.Num());

	const TSet<int32> DynamicVertexSet = DynamicSubMesh.GetVertices();
	DynamicVertices.Reserve(DynamicVertexSet.Num());
	for (const int32 PossiblyDynamicVertex : DynamicVertexSet)
	{
		// Some of these vertices may be kinematic since a triangle is dynamic if any of its vertices are.
		if (Particles.InvM(PossiblyDynamicVertex) > (FSolverReal)0.)
		{
			DynamicVertices.Add(PossiblyDynamicVertex);
		}
	}
}

void FPBDTriangleMeshCollisions::FTriangleSubMesh::InitAllDynamic()
{
	constexpr bool bCullDegenerateFalse = false;
	DynamicSubMesh.Init(FullMesh.GetElements(), FullMesh.GetVertexRange()[0], FullMesh.GetVertexRange()[1], bCullDegenerateFalse);
	KinematicColliderSubMesh.Init(TArray<TVec3<int32>>());

	FullMeshToSubMeshIndices.SetNumUninitialized(FullMesh.GetNumElements());
	DynamicSubMeshToFullMeshIndices.SetNumUninitialized(FullMesh.GetNumElements());
	KinematicColliderSubMeshToFullMeshIndices.Reset();
	DynamicVertices.Reset();
	for (int32 Index = 0; Index < FullMesh.GetNumElements(); ++Index)
	{
		FullMeshToSubMeshIndices[Index].SubMeshIndex = Index;
		FullMeshToSubMeshIndices[Index].SubMeshType = ESubMeshType::Dynamic;
		DynamicSubMeshToFullMeshIndices[Index] = Index;
	}
}

template<typename SolverParticlesOrRange>
void FPBDTriangleMeshCollisions::Init(const SolverParticlesOrRange& Particles, const FPBDFlatWeightMap& ThicknessMap)
{
	const bool bDoSelfIntersections = bGlobalIntersectionAnalysis || bContourMinimization;
	if (bCollidableSubMeshDirty)
	{		
		CollidableSubMesh.Init(Particles, DisabledFaces, bSelfCollideAgainstAllKinematicVertices, EnabledKinematicFaces, bOnlyCollideWithKinematics);
		bCollidableSubMeshDirty = false;
	}

	const FTriangleMesh& DynamicSubMesh = CollidableSubMesh.GetDynamicSubMesh();

	if (DynamicSubMesh.GetNumElements() == 0 && !bOnlyCollideWithKinematics)
	{
		return;
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ChaosFPBDTriangleMeshCollisions_BuildSpatialHash);

		constexpr FSolverReal RadiusToLodSizeMultiplier = 2.; // Radius to Diameter
		FSolverReal MinProximityQueryRadius;
		if (!ThicknessMap.HasWeightMap())
		{	
			MinProximityQueryRadius = ((FSolverReal)ThicknessMap) * 2.f; // When thickness is constant, we don't put any thickness in the spatial hash and instead query with particles with 2*thickness radius.
			DynamicSubMesh.BuildSpatialHash(static_cast<const TArrayView<const FSolverVec3>&>(Particles.XArray()), DynamicSubMeshSpatialHash, RadiusToLodSizeMultiplier * MinProximityQueryRadius);
		}
		else
		{
			MinProximityQueryRadius = FMath::Max(ThicknessMap.GetLow(), ThicknessMap.GetHigh());
			DynamicSubMesh.BuildSpatialHash(static_cast<const TArrayView<const FSolverVec3>&>(Particles.XArray()), DynamicSubMeshSpatialHash, ThicknessMap, Offset, RadiusToLodSizeMultiplier * MinProximityQueryRadius);
		}
		if (CollidableSubMesh.GetKinematicColliderSubMesh().GetNumElements() > 0)
		{
			CollidableSubMesh.GetKinematicColliderSubMesh().BuildSpatialHash(static_cast<const TArrayView<const FSolverVec3>&>(Particles.XArray()), KinematicSubMeshSpatialHash, RadiusToLodSizeMultiplier * MinProximityQueryRadius);
		}
	}
	ContourMinimizationIntersections.Reset();
	VertexGIAColors.Reset();
	VertexGIAColors.SetNumZeroed(NumParticles);
	TriangleGIAColors.Reset();
	TriangleGIAColors.SetNumZeroed(DynamicSubMesh.GetNumElements());
	IntersectionContourPoints.Reset();
	IntersectionContourTypes.Reset();

	if (!bDoSelfIntersections || bOnlyCollideWithKinematics)
	{
		return;
	}

	if (!ScratchBuffers)
	{
		ScratchBuffers = MakePimpl<FScratchBuffers>();
	}
	ScratchBuffers->Reset();

	// Detect all EdgeFace Intersections
	constexpr bool bSkipKinematicTrue = true;
	FindEdgeFaceIntersections(DynamicSubMesh, DynamicSubMeshSpatialHash, Particles, ScratchBuffers->EdgeFaceIntersections);
	if (ScratchBuffers->EdgeFaceIntersections.Num() == 0)
	{
		return;
	}

	if (bGlobalIntersectionAnalysis)
	{ 
		// Walk EdgeFace intersections to build global contours
		GIA::ContourBuilding::BuildIntersectionContours(DynamicSubMesh, ScratchBuffers->EdgeFaceIntersections, ScratchBuffers->IntersectionContours, IntersectionContourPoints);
		// Flood fill global contours to determine intersecting regions.
		GIA::FloodFill::FloodFillContours(DynamicSubMesh, NumParticles, Offset, ScratchBuffers->IntersectionContours, VertexGIAColors, TriangleGIAColors);
		GIA::FloodFill::AssignContourPointTypes(ScratchBuffers->IntersectionContours, IntersectionContourPoints, IntersectionContourTypes);
	}

	if (bContourMinimization)
	{
		if (bGlobalIntersectionAnalysis)
		{
			// Global contours which are non-closed or loop are handled via ContourMinimization impulses. Build global gradient (by adding contribution across all intersections per contour).
			ContourMinimization::BuildGlobalContourMinimizationIntersections(DynamicSubMesh, Particles, ScratchBuffers->IntersectionContours, ContourMinimizationIntersections);
		}
		else
		{
			// Just build local gradient for all Intersections
			ContourMinimization::BuildLocalContourMinimizationIntersections(DynamicSubMesh, Particles, ScratchBuffers->EdgeFaceIntersections, ContourMinimizationIntersections);
		}
	}
}
template CHAOS_API void FPBDTriangleMeshCollisions::Init(const FSolverParticles& Particles, const FPBDFlatWeightMap& ThicknessMap);
template CHAOS_API void FPBDTriangleMeshCollisions::Init(const FSolverParticlesRange& Particles, const FPBDFlatWeightMap& ThicknessMap);

template<typename SolverParticlesOrRange>
void FPBDTriangleMeshCollisions::InitFlesh(const SolverParticlesOrRange& Particles, const FPBDFlatWeightMap& ThicknessMap, const bool bUseFullMesh)
{
	const bool bDoSelfIntersections = bGlobalIntersectionAnalysis || bContourMinimization;
	if (bCollidableSubMeshDirty)
	{		
		CollidableSubMesh.Init(Particles, DisabledFaces, bSelfCollideAgainstAllKinematicVertices, EnabledKinematicFaces, bOnlyCollideWithKinematics);
		bCollidableSubMeshDirty = false;
	}

	if (CollidableSubMesh.GetDynamicSubMesh().GetNumElements() == 0 && CollidableSubMesh.GetFullMesh().GetNumElements() == 0)
	{
		return;
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ChaosFPBDTriangleMeshCollisions_BuildSpatialHash);

		constexpr FSolverReal RadiusToLodSizeMultiplier = 2.; // Radius to Diameter
		FSolverReal MinProximityQueryRadius;
		if (!ThicknessMap.HasWeightMap())
		{	
			MinProximityQueryRadius = ((FSolverReal)ThicknessMap) * 2.f; // When thickness is constant, we don't put any thickness in the spatial hash and instead query with particles with 2*thickness radius.
			if (bUseFullMesh)
			{
				CollidableSubMesh.GetFullMesh().BuildSpatialHash(static_cast<const TArrayView<const FSolverVec3>&>(Particles.XArray()), DynamicSubMeshSpatialHash, RadiusToLodSizeMultiplier * MinProximityQueryRadius);
			}
			else
			{
				CollidableSubMesh.GetDynamicSubMesh().BuildSpatialHash(static_cast<const TArrayView<const FSolverVec3>&>(Particles.XArray()), DynamicSubMeshSpatialHash, RadiusToLodSizeMultiplier * MinProximityQueryRadius);
			}
		}
		else
		{
			MinProximityQueryRadius = FMath::Max(ThicknessMap.GetLow(), ThicknessMap.GetHigh());
			if (bUseFullMesh)
			{
				CollidableSubMesh.GetFullMesh().BuildSpatialHash(static_cast<const TArrayView<const FSolverVec3>&>(Particles.XArray()), DynamicSubMeshSpatialHash, ThicknessMap, Offset, RadiusToLodSizeMultiplier * MinProximityQueryRadius);
			}
			else
			{
				CollidableSubMesh.GetDynamicSubMesh().BuildSpatialHash(static_cast<const TArrayView<const FSolverVec3>&>(Particles.XArray()), DynamicSubMeshSpatialHash, ThicknessMap, Offset, RadiusToLodSizeMultiplier * MinProximityQueryRadius);
			}
		}
	}
}
template CHAOS_API void FPBDTriangleMeshCollisions::InitFlesh(const FSolverParticles& Particles, const FPBDFlatWeightMap& ThicknessMap, const bool bUseFullMesh);
template CHAOS_API void FPBDTriangleMeshCollisions::InitFlesh(const FSolverParticlesRange& Particles, const FPBDFlatWeightMap& ThicknessMap, const bool bUseFullMesh);

template<typename SolverParticlesOrRange>
void FPBDTriangleMeshCollisions::PostStepInit(const SolverParticlesOrRange& Particles)
{
	const FTriangleMesh& DynamicSubMesh = CollidableSubMesh.GetDynamicSubMesh();
	if (DynamicSubMesh.GetNumElements() == 0)
	{
		return;
	}

	PostStepIntersectionContourPoints.Reset();
	PostStepContourMinimizationIntersections.Reset();

	if (NumContourMinimizationPostSteps > 0)
	{
		// For now just going to rebuild the spatial grid every time. In reality, should really do some sort of refitting.
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ChaosFPBDTriangleMeshCollisions_BuildSpatialHash);
			constexpr FSolverReal MinSpatialLodSize = 1.f;
			DynamicSubMesh.BuildSpatialHash(static_cast<const TArrayView<const FSolverVec3>&>(Particles.XArray()), DynamicSubMeshSpatialHash, MinSpatialLodSize);

			if (!ScratchBuffers)
			{
				ScratchBuffers = MakePimpl<FScratchBuffers>();
			}
			ScratchBuffers->Reset();

			// Detect all EdgeFace Intersections
			FindEdgeFaceIntersections(DynamicSubMesh, DynamicSubMeshSpatialHash, Particles, ScratchBuffers->EdgeFaceIntersections);

			if (bUseGlobalPostStepContours)
			{
				GIA::ContourBuilding::BuildIntersectionContours(DynamicSubMesh, ScratchBuffers->EdgeFaceIntersections, ScratchBuffers->IntersectionContours, PostStepIntersectionContourPoints);
				ContourMinimization::BuildGlobalContourMinimizationIntersections(DynamicSubMesh, Particles, ScratchBuffers->IntersectionContours, PostStepContourMinimizationIntersections);
			}
			else
			{
				// Just build local gradient for all Intersections
				ContourMinimization::BuildLocalContourMinimizationIntersections(DynamicSubMesh, Particles, ScratchBuffers->EdgeFaceIntersections, PostStepContourMinimizationIntersections);
			}
		}
	}
}
template CHAOS_API void FPBDTriangleMeshCollisions::PostStepInit(const FSolverParticles& Particles);
template CHAOS_API void FPBDTriangleMeshCollisions::PostStepInit(const FSolverParticlesRange& Particles);
}  // End namespace Chaos::Softs
