// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/PBDTriangleMeshIntersections.h"
#include "Chaos/PBDSoftsSolverParticles.h"
#include "Chaos/SoftsSolverParticlesRange.h"

float Chaos_TriangleIntersections_MaxDelta = 0.01f;
FAutoConsoleVariableRef CVarChaosTriangleIntersectionMaxImpulse(TEXT("p.Chaos.TriangleIntersections.MaxDelta"), Chaos_TriangleIntersections_MaxDelta, TEXT("Maximum delta position applied to resolve triangle intersections."));

namespace Chaos::Softs {

static inline FSolverVec3 GetDelta(const FPBDTriangleMeshCollisions::FContourMinimizationIntersection& Intersection, const FSolverReal MaxDelta, const FSolverReal RegularizeEpsilonSq)
{
	// Using pre-calculated "GlobalGradientVector". Whether this is actually globally calculated or a copy of the local gradient depends on settings in PBDTriangleMeshCollisions (default is global)
	FSolverReal GradientLength;
	FSolverVec3 GradientDir;
	Intersection.GlobalGradientVector.ToDirectionAndLength(GradientDir, GradientLength);

	return MaxDelta * GradientLength * FMath::InvSqrt(GradientLength * GradientLength + RegularizeEpsilonSq) * GradientDir;
}

template<typename SolverParticlesOrRange>
void FPBDTriangleMeshIntersections::Apply(SolverParticlesOrRange& Particles, const TArray<FPBDTriangleMeshCollisions::FContourMinimizationIntersection>& Intersections, const FSolverReal Dt) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ChaosFPBDTriangleMeshIntersections_Apply);
	static FSolverReal RegularizeEpsilonSq = 1.f;
	for( const FPBDTriangleMeshCollisions::FContourMinimizationIntersection& Intersection : Intersections )
	{
		const FSolverVec3 Delta = GetDelta(Intersection, Chaos_TriangleIntersections_MaxDelta, RegularizeEpsilonSq);
		if (Particles.InvM(Intersection.EdgeVertices[0]) > 0)
		{
			Particles.P(Intersection.EdgeVertices[0]) += Delta;
		}
		if (Particles.InvM(Intersection.EdgeVertices[1]) > 0)
		{
			Particles.P(Intersection.EdgeVertices[1]) += Delta;
		}
		if (Particles.InvM(Intersection.FaceVertices[0]) > 0)
		{
			Particles.P(Intersection.FaceVertices[0]) -= Delta;
		}
		if (Particles.InvM(Intersection.FaceVertices[1]) > 0)
		{
			Particles.P(Intersection.FaceVertices[1]) -= Delta;
		}
		if (Particles.InvM(Intersection.FaceVertices[2]) > 0)
		{
			Particles.P(Intersection.FaceVertices[2]) -= Delta;
		}
	}
}
template CHAOS_API void FPBDTriangleMeshIntersections::Apply(FSolverParticles& Particles, const TArray<FPBDTriangleMeshCollisions::FContourMinimizationIntersection>& Intersections, const FSolverReal Dt) const;
template CHAOS_API void FPBDTriangleMeshIntersections::Apply(FSolverParticlesRange& Particles, const TArray<FPBDTriangleMeshCollisions::FContourMinimizationIntersection>& Intersections, const FSolverReal Dt) const;

}  // End namespace Chaos::Softs
