// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/PBDUnilateralTetConstraints.h"
#include "Chaos/GraphColoring.h"
#include "Chaos/Triangle.h"
#include "ChaosStats.h"
#include "ChaosLog.h"
#include "HAL/IConsoleManager.h"
#if 0 && INTEL_ISPC
#include "PBDUnilateralTetConstraints.ispc.generated.h"
#endif

#if 0 && INTEL_ISPC

#if !defined(CHAOS_UNILATERAL_TET_ISPC_ENABLED_DEFAULT)
#define CHAOS_UNILATERAL_TET_ISPC_ENABLED_DEFAULT 1
#endif

#if !defined(USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING)
#define USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING 0
#endif

// Support run-time toggling on supported platforms in non-shipping configurations
#if (UE_BUILD_SHIPPING && !USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING)
static constexpr bool bChaos_UnilateralTet_ISPC_Enabled = INTEL_ISPC && CHAOS_UNILATERAL_TET_ISPC_ENABLED_DEFAULT;
#else

static bool bChaos_UnilateralTet_ISPC_Enabled;

#endif

static_assert(sizeof(ispc::FVector4f) == sizeof(Chaos::Softs::FPAndInvM), "sizeof(ispc::FVector4f) != sizeof(Chaos::Softs::FPAndInvM)");
static_assert(sizeof(ispc::FIntVector4) == sizeof(Chaos::TVec4<int32>), "sizeof(ispc::FIntVector4) != sizeof(Chaos::TVec4<int32>");

#endif
namespace Chaos::Softs
{
	FPBDUnilateralTetConstraints::FPBDUnilateralTetConstraints(
		const FSolverParticlesRange& Particles,
		TArray<TVector<int32, 4>>&& InConstraints,
		TArray<FSolverReal>&& InVolumes,
		FSolverReal InStiffness,
		int32 InMaxNumIters)
		: Constraints(MoveTemp(InConstraints))
		, Volumes(MoveTemp(InVolumes))
		, Stiffness(InStiffness)
		, MaxNumIters(InMaxNumIters)
	{
		TrimKinematicConstraints(Particles);
		InitColor(Particles);

#if CHAOS_DEBUG_DRAW
		ConstraintIsActive.SetNum(Constraints.Num());
#endif
	}

	void FPBDUnilateralTetConstraints::TrimKinematicConstraints(const FSolverParticlesRange& Particles)
	{
		TArray<TVector<int32, 4>> TrimmedConstraints;
		TArray<FSolverReal> TrimmedVolumes;
		TrimmedConstraints.Reserve(Constraints.Num());
		TrimmedVolumes.Reserve(Constraints.Num());

		for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
		{
			if (Particles.InvM(Constraints[ConstraintIndex][0]) != (FSolverReal)0.f ||
				Particles.InvM(Constraints[ConstraintIndex][1]) != (FSolverReal)0.f ||
				Particles.InvM(Constraints[ConstraintIndex][2]) != (FSolverReal)0.f ||
				Particles.InvM(Constraints[ConstraintIndex][3]) != (FSolverReal)0.f)
			{
				TrimmedConstraints.Add(Constraints[ConstraintIndex]);
				TrimmedVolumes.Add(Volumes[ConstraintIndex]);
			}
		}

		TrimmedConstraints.Shrink();
		TrimmedVolumes.Shrink();

		Constraints = MoveTemp(TrimmedConstraints);
		Volumes = MoveTemp(TrimmedVolumes);
	}

	void FPBDUnilateralTetConstraints::InitColor(const FSolverParticlesRange& Particles)
	{
		const int32 NumConstraints = Constraints.Num();
		const TArray<TArray<int32>> ConstraintsPerColor = FGraphColoring::ComputeGraphColoringParticlesOrRange(Constraints, Particles, 0, Particles.Size());
		// Reorder constraints based on color so each array in ConstraintsPerColor contains contiguous elements.
		TArray<TVector<int32, 4>> ReorderedConstraints;
		TArray<FSolverReal> ReorderedRestLengths;
		TArray<FSolverReal> ReorderedVolumes;
		ReorderedConstraints.SetNumUninitialized(NumConstraints);
		ReorderedRestLengths.SetNumUninitialized(NumConstraints);
		ReorderedVolumes.SetNumUninitialized(NumConstraints);

		ConstraintsPerColorStartIndex.Reset(ConstraintsPerColor.Num() + 1);
		int32 ReorderedIndex = 0;
		for (const TArray<int32>& ConstraintsBatch : ConstraintsPerColor)
		{
			ConstraintsPerColorStartIndex.Add(ReorderedIndex);
			for (const int32 OrigIndex : ConstraintsBatch)
			{
				ReorderedConstraints[ReorderedIndex] = Constraints[OrigIndex];
				ReorderedVolumes[ReorderedIndex] = Volumes[OrigIndex];

				++ReorderedIndex;
			}
		}
		ConstraintsPerColorStartIndex.Add(ReorderedIndex);

		Constraints = MoveTemp(ReorderedConstraints);
		Volumes = MoveTemp(ReorderedVolumes);
	}
	
	void FPBDUnilateralTetConstraints::Apply(FSolverParticlesRange& Particles, const FSolverReal Dt) const
	{
		ApplyVolumeConstraint(Particles, Dt);
	}

	void FPBDUnilateralTetConstraints::ApplyVolumeConstraint(FSolverParticlesRange& Particles, const FSolverReal Dt) const
	{
		// TODO ISPC
		const TVec4<int32>* const ConstraintsData = Constraints.GetData();

		FPAndInvM* const PAndInvM = Particles.GetPAndInvM().GetData();

		for (int32 Iter = 0; Iter < MaxNumIters; ++Iter)
		{
			bool bAnyActive = false;
			for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
			{
#if CHAOS_DEBUG_DRAW
				ConstraintIsActive[ConstraintIndex] = false;
#endif
				const TVec4<int32>& Constraint = ConstraintsData[ConstraintIndex];
				const int32 Index1 = Constraint[1];
				const int32 Index2 = Constraint[2];
				const int32 Index3 = Constraint[3];
				const int32 Index4 = Constraint[0];
				FSolverVec3& P1 = PAndInvM[Index1].P;
				FSolverVec3& P2 = PAndInvM[Index2].P;
				FSolverVec3& P3 = PAndInvM[Index3].P;
				FSolverVec3& P4 = PAndInvM[Index4].P;

				TVec4<FSolverVec3> Grads;
				const FSolverVec3 P2P1 = P2 - P1;
				const FSolverVec3 P4P1 = P4 - P1;
				const FSolverVec3 P3P1 = P3 - P1;
				Grads[1] = FSolverVec3::CrossProduct(P3P1, P4P1) / (FSolverReal)6.;
				Grads[2] = FSolverVec3::CrossProduct(P4P1, P2P1) / (FSolverReal)6.;
				Grads[3] = FSolverVec3::CrossProduct(P2P1, P3P1) / (FSolverReal)6.;
				Grads[0] = -(Grads[1] + Grads[2] + Grads[3]);

				const FSolverReal Volume = FSolverVec3::DotProduct(FSolverVec3::CrossProduct(P2P1, P3P1), P4P1) / (FSolverReal)6.;
				if ((Volume - Volumes[ConstraintIndex]) >= 0)
				{
					continue;
				}

				const FSolverReal S = Stiffness * (Volume - Volumes[ConstraintIndex]) / (
					PAndInvM[Index1].InvM * Grads[0].SizeSquared() +
					PAndInvM[Index2].InvM * Grads[1].SizeSquared() +
					PAndInvM[Index3].InvM * Grads[2].SizeSquared() +
					PAndInvM[Index4].InvM * Grads[3].SizeSquared());

				P1 -= S * PAndInvM[Index1].InvM * Grads[0];
				P2 -= S * PAndInvM[Index2].InvM * Grads[1];
				P3 -= S * PAndInvM[Index3].InvM * Grads[2];
				P4 -= S * PAndInvM[Index4].InvM * Grads[3];

				bAnyActive = true;
#if CHAOS_DEBUG_DRAW
				ConstraintIsActive[ConstraintIndex] = true;
#endif
			}
			if (!bAnyActive)
			{
				break;
			}
		}
	}


	FPBDVertexFaceRepulsionConstraints::FPBDVertexFaceRepulsionConstraints(
		const FSolverParticlesRange& Particles,
		const FCollectionPropertyConstFacade& PropertyCollection,
		const FEmbeddedSpringConstraintFacade& SpringConstraintFacade)
		: FPBDUnilateralTetConstraints(
			Particles
			, ExtractConstraintIndices(SpringConstraintFacade)
			, ExtractVolumes(Particles, SpringConstraintFacade)
			, FMath::Clamp(GetVertexFaceRepulsionStiffness(PropertyCollection, DefaultStiffness), MinStiffness, MaxStiffness)
			, FMath::Max(GetVertexFaceMaxRepulsionIters(PropertyCollection, 1), 1)
		)
		, VertexFaceRepulsionStiffnessIndex(PropertyCollection)
		, VertexFaceMaxRepulsionItersIndex(PropertyCollection)
	{
	}

	void FPBDVertexFaceRepulsionConstraints::SetProperties(const FCollectionPropertyConstFacade& PropertyCollection)
	{
		if (IsVertexFaceRepulsionStiffnessMutable(PropertyCollection))
		{
			Stiffness = FMath::Clamp(GetVertexFaceRepulsionStiffness(PropertyCollection), MinStiffness, MaxStiffness);
		}
		if (IsVertexFaceMaxRepulsionItersMutable(PropertyCollection))
		{
			MaxNumIters = FMath::Max(GetVertexFaceMaxRepulsionIters(PropertyCollection), 1);
		}
	}

	TArray<TVector<int32, 4>> FPBDVertexFaceRepulsionConstraints::ExtractConstraintIndices(const FEmbeddedSpringConstraintFacade& SpringConstraintFacade)
	{
		const TConstArrayView<TArray<int32>>& InSourceIndices = SpringConstraintFacade.GetSourceIndexConst();
		const TConstArrayView<TArray<int32>>& InTargetIndices = SpringConstraintFacade.GetTargetIndexConst();

		const int32 NumConstraints = InSourceIndices.Num();
		check(InTargetIndices.Num() == NumConstraints);

		TArray<TVector<int32,4>> ResultConstraints;
		ResultConstraints.Reserve(NumConstraints);

		for (int32 ConstraintIdx = 0; ConstraintIdx < NumConstraints; ++ConstraintIdx)
		{
			ResultConstraints.Emplace(InSourceIndices[ConstraintIdx][0], InTargetIndices[ConstraintIdx][0], InTargetIndices[ConstraintIdx][1], InTargetIndices[ConstraintIdx][2]);
		}
		return ResultConstraints;
	}

	TArray<FSolverReal> FPBDVertexFaceRepulsionConstraints::ExtractVolumes(const FSolverParticlesRange& Particles, const FEmbeddedSpringConstraintFacade& SpringConstraintFacade)
	{
		const TConstArrayView<TArray<int32>>& InTargetIndices = SpringConstraintFacade.GetTargetIndexConst();
		const TConstArrayView<float>& InSpringLengths = SpringConstraintFacade.GetSpringLengthConst();

		const int32 NumConstraints = InTargetIndices.Num();
		check(InSpringLengths.Num() == NumConstraints);

		TArray<FSolverReal> ResultVolumes;
		ResultVolumes.Reserve(NumConstraints);

		for (int32 ConstraintIdx = 0; ConstraintIdx < NumConstraints; ++ConstraintIdx)
		{
			const TArray<int32>& Constraint = InTargetIndices[ConstraintIdx];
			// Calculate Volume = area of triangle * restlength.
			const FSolverReal Volume =
				(FSolverVec3::CrossProduct(Particles.X(Constraint[1]) - Particles.X(Constraint[0]), Particles.X(Constraint[2]) - Particles.X(Constraint[0]))).Length()
				* InSpringLengths[ConstraintIdx] / (FSolverReal)6.;

			ResultVolumes.Emplace(Volume);
		}
		return ResultVolumes;
	}
} // End namespace Chaos::Softs
