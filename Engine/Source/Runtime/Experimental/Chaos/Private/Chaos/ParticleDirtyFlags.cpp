// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/ParticleDirtyFlags.h"

namespace Chaos
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// The new instance data and shape filter data are stored in the old Query/Sim Data until deprecation allows removing the old fields.
	static_assert(sizeof(FCollisionFilterData::Word0) + sizeof(FCollisionFilterData::Word1) == sizeof(Filter::FInstanceData));
	constexpr size_t ShapeFilterPart1Size = sizeof(FCollisionFilterData::Word2) + sizeof(FCollisionFilterData::Word3);
	constexpr size_t ShapeFilterPart2Size = sizeof(Filter::FShapeFilterData) - ShapeFilterPart1Size;
	static_assert(ShapeFilterPart2Size <= sizeof(FCollisionFilterData));
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	FCollisionFilterData FCollisionData::GetQueryData() const
	{
		return Filter::FShapeFilterBuilder::GetLegacyShapeQueryFilter(GetCombinedShapeFilterData());
	}

	void FCollisionData::SetQueryData(const FCollisionFilterData& InQueryData)
	{
		Filter::FCombinedShapeFilterData CombinedFilterData = GetCombinedShapeFilterData();
		Filter::FShapeFilterBuilder::SetLegacyShapeQueryFilter(CombinedFilterData, InQueryData);
		SetCombinedShapeFilterData(CombinedFilterData);
	}

	FCollisionFilterData FCollisionData::GetSimData() const
	{
		return Filter::FShapeFilterBuilder::GetLegacyShapeSimFilter(GetCombinedShapeFilterData());
	}

	void FCollisionData::SetSimData(const FCollisionFilterData& InSimData)
	{
		Filter::FCombinedShapeFilterData CombinedFilterData = GetCombinedShapeFilterData();
		Filter::FShapeFilterBuilder::SetLegacyShapeSimFilter(CombinedFilterData, InSimData);
		SetCombinedShapeFilterData(CombinedFilterData);
	}

	Chaos::Filter::FShapeFilterData FCollisionData::GetShapeFilterData() const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Filter::FShapeFilterData ShapeFilter;
		ShapeFilter.Load(QueryData, SimData);
		return ShapeFilter;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void FCollisionData::SetShapeFilterData(const Chaos::Filter::FShapeFilterData& ShapeFilter)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ShapeFilter.Store(QueryData, SimData);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	Chaos::Filter::FInstanceData FCollisionData::GetFilterInstanceData() const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Filter::FInstanceData InstanceData;
		memcpy(&InstanceData, &QueryData, sizeof(Filter::FInstanceData));
		return InstanceData;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void FCollisionData::SetFilterInstanceData(const Chaos::Filter::FInstanceData& InstanceData)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		memcpy(&QueryData, &InstanceData, sizeof(Filter::FInstanceData));
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	Chaos::Filter::FCombinedShapeFilterData FCollisionData::GetCombinedShapeFilterData() const
	{
		const Filter::FInstanceData& InstanceData = GetFilterInstanceData();
		const Filter::FShapeFilterData& ShapeFilter = GetShapeFilterData();
		return Filter::FCombinedShapeFilterData(ShapeFilter, InstanceData);
	}

	void FCollisionData::SetCombinedShapeFilterData(const Chaos::Filter::FCombinedShapeFilterData& CombinedShapeFilter)
	{
		SetFilterInstanceData(CombinedShapeFilter.GetInstanceData());
		SetShapeFilterData(CombinedShapeFilter.GetShapeFilterData());
	}
} // namespace Chaos
