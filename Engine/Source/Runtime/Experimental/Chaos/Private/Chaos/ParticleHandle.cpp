// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/ParticleHandle.h"

#include "Chaos/ImplicitObjectTransformed.h"
#include "Chaos/ImplicitObjectScaled.h"
#include "Chaos/ImplicitObjectUnion.h"
#include "Chaos/TriangleMeshImplicitObject.h"
#include "Chaos/Framework/PhysicsProxy.h"
#include "Chaos/Framework/PhysicsSolverBase.h"
#include "Chaos/Framework/PhysicsProxyBase.h"
#include "Chaos/CastingUtilities.h"
#include "PBDRigidsSolver.h"

namespace Chaos
{
	namespace CVars
	{
		static bool GForceDeepCopyOnModifyGeometry = false;

		FAutoConsoleVariableRef CVarForceDeepCopyOnModifyGeometry(TEXT("p.Chaos.Geometry.ForceDeepCopyAccess"), GForceDeepCopyOnModifyGeometry, TEXT("Whether we always use a deep copy when modifying particle geometry"));

		bool ForceDeepCopyOnModifyGeometry()
		{
			return GForceDeepCopyOnModifyGeometry;
		}
	}

	extern void UpdateShapesArrayFromGeometry(FShapeInstanceProxyArray& ShapesArray, const FImplicitObjectPtr& Geometry, const FRigidTransform3& ActorTM, IPhysicsProxyBase* Proxy);

	void SetObjectStateHelper(IPhysicsProxyBase& Proxy, FPBDRigidParticleHandle& Rigid, EObjectStateType InState, bool bAllowEvents, bool bInvalidate)
	{
		if (auto PhysicsSolver = Proxy.GetSolver<Chaos::FPBDRigidsSolver>())
		{
			PhysicsSolver->GetEvolution()->SetParticleObjectState(&Rigid, InState);
		}
		else
		{
			//not in solver so just set it directly (can this possibly happen?)
			Rigid.SetObjectStateLowLevel(InState);
		}
	}

	inline FImplicitObject* GetInstancedImplicitHelper(FImplicitObject* Implicit0)
	{
		EImplicitObjectType Implicit0OuterType = Implicit0->GetType();

		if (Implicit0OuterType == TImplicitObjectInstanced<FConvex>::StaticType())
		{
			return const_cast<FConvex*>(Implicit0->template GetObject<TImplicitObjectInstanced<FConvex>>()->GetInstancedObject());
		}
		else if (Implicit0OuterType == TImplicitObjectInstanced<TBox<FReal, 3>>::StaticType())
		{
			return const_cast<TBox<FReal, 3>*>(Implicit0->template GetObject<TImplicitObjectInstanced<TBox<FReal, 3>>>()->GetInstancedObject());
		}
		else if (Implicit0OuterType == TImplicitObjectInstanced<FCapsule>::StaticType())
		{
			return const_cast<FCapsule*>(Implicit0->template GetObject<TImplicitObjectInstanced<FCapsule>>()->GetInstancedObject());
		}
		else if (Implicit0OuterType == TImplicitObjectInstanced<FSphere>::StaticType())
		{
			return const_cast<FSphere*>(Implicit0->template GetObject<TImplicitObjectInstanced<FSphere>>()->GetInstancedObject());
		}
		else if (Implicit0OuterType == TImplicitObjectInstanced<FTriangleMeshImplicitObject>::StaticType())
		{
			return const_cast<FTriangleMeshImplicitObject*>(Implicit0->template GetObject<TImplicitObjectInstanced<FTriangleMeshImplicitObject>>()->GetInstancedObject());
		}

		return nullptr;
	}

	template <typename T, int d>
	void Chaos::TGeometryParticle<T, d>::UpdateShapesArray()
	{
		UpdateShapesArrayFromGeometry(MShapesArray, MNonFrequentData.Read().GetGeometry(), FRigidTransform3(X(), R()), Proxy);
	}

	template <typename T, int d>
	void Chaos::TGeometryParticle<T, d>::MergeGeometry(TArray<Chaos::FImplicitObjectPtr>&& Objects)
	{
		// We always have a union at the root. Add one if these are the first objects to be added
		if (MNonFrequentData.Read().GetGeometry() == nullptr)
		{
			Chaos::FImplicitObjectPtr Union = MakeImplicitObjectPtr<Chaos::FImplicitObjectUnion>(MoveTemp(Objects));
			SetGeometry(Union);
			return;
		}

		// we only support FImplicitObjectUnion
		check(MNonFrequentData.Read().GetGeometry());
		if (ensure(MNonFrequentData.Read().GetGeometry()->GetType() == FImplicitObjectUnion::StaticType()))
		{
			// Only adding to the root union - shallow copy allowed here.
			ModifyGeometry(EGeometryAccess::ShallowCopy,
				[&Objects, this](FImplicitObject& GeomToModify)
				{
					if (FImplicitObjectUnion* Union = GeomToModify.template GetObject<FImplicitObjectUnion>())
					{
						Union->Combine(Objects);
					}
				});
		}
	}

	template <typename T, int d, bool bPersistent>
	void Chaos::TGeometryParticleHandleImp<T, d, bPersistent>::MergeGeometry(TArray<Chaos::FImplicitObjectPtr>&& Objects)
	{
		if (Objects.IsEmpty())
		{
			return;
		}

		// We always have a union at the root. Add one if these are the first objects to be added
		const FImplicitObjectRef CurrentGeometry = GetGeometry();
		if (CurrentGeometry == nullptr)
		{
			Chaos::FImplicitObjectPtr Union = MakeImplicitObjectPtr<Chaos::FImplicitObjectUnion>(MoveTemp(Objects));
			SetGeometry(Union);
		}
		else
		{
			if (ensure(CurrentGeometry->GetType() == FImplicitObjectUnion::StaticType()))
			{
				FImplicitObjectUnion& Union = CurrentGeometry->GetObjectChecked<FImplicitObjectUnion>();
				Union.Combine(Objects);

				// Needed to update the shapes array.
				SetGeometry(GeometryParticles->GetGeometry(ParticleIdx));

				CVD_TRACE_INVALIDATE_CACHED_GEOMETRY(CurrentGeometry);
			}
		}
	}

	template <typename T, int d, bool bPersistent>
	void Chaos::TGeometryParticleHandleImp<T, d, bPersistent>::LockGeometry()
	{
		if (FImplicitObjectRef CurrentGeometry = GetGeometry())
		{
			if (FImplicitObjectUnion* Union = CurrentGeometry->AsA<FImplicitObjectUnion>())
			{
				Union->SetLockedRecursive(true);
			}
		}
	}

	template <typename T, int d>
	void Chaos::TGeometryParticle<T, d>::RemoveShape(FPerShapeData* InShape, bool bWakeTouching)
	{
		// NOTE: only intended use is to remove objects from inside a FImplicitObjectUnion
		CHAOS_ENSURE(MNonFrequentData.Read().GetGeometry()->GetType() == FImplicitObjectUnion::StaticType());

		for (int32 Index = 0; Index < MShapesArray.Num(); Index++)
		{
			if (InShape == MShapesArray[Index].Get())
			{
				RemoveShapesAtSortedIndices({ Index });
				return;
			}
		}
	}

	template <typename T, int d>
	void Chaos::TGeometryParticle<T, d>::RemoveShapesAtSortedIndices(const TArrayView<const int32>& InIndices)
	{
		// NOTE: only intended use is to remove objects from inside a FImplicitObjectUnion
		CHAOS_ENSURE(MNonFrequentData.Read().GetGeometry()->GetType() == FImplicitObjectUnion::StaticType());

		// Only removing shapes, shallow copy is allowed
		ModifyGeometry(EGeometryAccess::ShallowCopy,
			[this, &InIndices](FImplicitObject& GeomToModify)
			{
				if (FImplicitObjectUnion* Union = GeomToModify.template AsA<FImplicitObjectUnion>())
				{
					RemoveArrayItemsAtSortedIndices(MShapesArray, InIndices);

					Union->RemoveAtSortedIndices(InIndices);

					// If shapes are removed in the middle of the array, the logic in UpdateShapesArrayFromGeometry does 
					// not actually pack down the indices correctly as we've already removed the shape. 
					// Just force the indices to be correct here.
					if (!InIndices.IsEmpty())
					{
						// These indices are sorted, we can start the fix-up at the first index.
						const int32 FirstIndex = InIndices[0];
						for (int32 I = FirstIndex; I < MShapesArray.Num(); ++I)
						{
							MShapesArray[I]->ModifyShapeIndex(I);
						}
					}
				}
			});
	}

	template <typename T, int d, bool bPersistent>
	void Chaos::TGeometryParticleHandleImp<T, d, bPersistent>::RemoveShape(FPerShapeData* InShape)
	{
		// NOTE: only intended use is to remove objects from inside a FImplicitObjectUnion
		const FImplicitObjectRef CurrentGeometry = GetGeometry();
		if (ensure(CurrentGeometry != nullptr))
		{
			const FShapesArray& CurrentShapesArray = ShapesArray();
			for (int32 Index = 0; Index < CurrentShapesArray.Num(); Index++)
			{
				if (InShape == CurrentShapesArray[Index].Get())
				{
					RemoveShapesAtSortedIndices(MakeArrayView({ Index }));
					return;
				}
			}

			CVD_TRACE_INVALIDATE_CACHED_GEOMETRY(CurrentGeometry);
		}
	}

	template <typename T, int d, bool bPersistent>
	void Chaos::TGeometryParticleHandleImp<T, d, bPersistent>::RemoveShapesAtSortedIndices(const TArrayView<const int32>& InIndices)
	{
		// NOTE: only intended use is to remove objects from inside a FImplicitObjectUnion
		const FImplicitObjectRef CurrentGeometry = GetGeometry();
		if (CurrentGeometry == nullptr)
		{
			return;
		}

		FImplicitObjectUnion* Union = CurrentGeometry->template AsA<FImplicitObjectUnion>();
		if (Union == nullptr)
		{
			return;
		}

		GeometryParticles->RemoveShapesAtSortedIndices(ParticleIdx, InIndices);

		Union->RemoveAtSortedIndices(InIndices);

		CVD_TRACE_INVALIDATE_CACHED_GEOMETRY(CurrentGeometry);

		// Needed to update the shapes array.
		// @todo(chaos): is it though? Maybe for the bounds etc?
		SetGeometry(GeometryParticles->GetGeometry(ParticleIdx));
	}

	template <typename T, int d>
	void Chaos::TGeometryParticle<T, d>::PrepareBVHImpl()
	{
		if (MNonFrequentData.IsDirty(MDirtyFlags))
		{
			if (FImplicitObjectRef Implicit = MNonFrequentData.Read().GetGeometry())
			{
				if (const FImplicitObjectUnion* Union = Implicit->AsA<FImplicitObjectUnion>())
				{
					// This will rebuild the BVH if the geometry is new, otherwise do nothing
					const_cast<FImplicitObjectUnion*>(Union)->SetAllowBVH(true);
				}
			}
		}
	}

	template <typename T, int d>
	void Chaos::TGeometryParticle<T, d>::LockGeometry()
	{
		if (MNonFrequentData.IsDirty(MDirtyFlags))
		{
			if (FImplicitObjectRef Implicit = MNonFrequentData.Read().GetGeometry())
			{
				if (FImplicitObjectUnion* Union = Implicit->AsA<FImplicitObjectUnion>())
				{
					Union->SetLockedRecursive(true);
				}
			}
		}
	}

	template <typename T, int d>
	void Chaos::TGeometryParticle<T, d>::SetIgnoreAnalyticCollisionsImp(FImplicitObject* Implicit, bool bIgnoreAnalyticCollisions)
	{
		check(Implicit);
		if (Implicit->GetType() == FImplicitObjectUnion::StaticType())
		{
			FImplicitObjectUnion* Union = Implicit->template GetObject<FImplicitObjectUnion>();
			for (const auto& Child : Union->GetObjects())
			{
				SetIgnoreAnalyticCollisionsImp(Child.GetReference(), bIgnoreAnalyticCollisions);
			}
		}
		else if (Implicit->GetType() == TImplicitObjectTransformed<T, d>::StaticType())
		{
			TImplicitObjectTransformed<FReal, 3>* TransformedImplicit = Implicit->template GetObject<TImplicitObjectTransformed<FReal, 3>>();
			SetIgnoreAnalyticCollisionsImp(const_cast<FImplicitObject*>(TransformedImplicit->GetTransformedObject()), bIgnoreAnalyticCollisions);
		}
		else if ((uint32)Implicit->GetType() & ImplicitObjectType::IsInstanced)
		{
			SetIgnoreAnalyticCollisionsImp(GetInstancedImplicitHelper(Implicit), bIgnoreAnalyticCollisions);
		}
		else
		{

			// Find our shape and see if sim is enabled.
			for (const TUniquePtr<FPerShapeData>& Shape : ShapesArray())
			{
				if (Shape->GetGeometry() == Implicit) 
				{
					if (!Shape->GetSimEnabled())
					{
						return;
					}
					break;
				}
			}
			if (bIgnoreAnalyticCollisions)
			{
				Implicit->SetCollisionType(Chaos::ImplicitObjectType::LevelSet);
				//Implicit->SetConvex(false);
			}
			else
			{
				Implicit->SetCollisionType(Implicit->GetType());
				// @todo (mlentine): Need to in theory set convex properly here
			}
		}
	}

	template <typename T, int d>
	void TGeometryParticle<T,d>::SetIgnoreAnalyticCollisions(bool bIgnoreAnalyticCollisions)
	{
		// Deep copy required as we modify the actual geometries
		ModifyGeometry(EGeometryAccess::DeepCopy,
			[this, bIgnoreAnalyticCollisions](FImplicitObject& GeomToModify)
			{
				SetIgnoreAnalyticCollisionsImp(&GeomToModify, bIgnoreAnalyticCollisions);
			});
	}

	template <typename T, int d, bool bPersistent>
	void TPBDRigidParticleHandleImp<T, d, bPersistent>::AddTorque(const TVector<T, d>& InTorque, bool bInvalidate)
	{
		const FMatrix33 WorldInvI = Utilities::ComputeWorldSpaceInertia(QCom(), InvI());
		SetAngularAcceleration(AngularAcceleration() + WorldInvI * InTorque);
	}


	template <typename T, int d, bool bPersistent>
	void TPBDRigidParticleHandleImp<T, d, bPersistent>::SetTorque(const TVector<T, d>& InTorque, bool bInvalidate)
	{
		const FMatrix33 WorldInvI = Utilities::ComputeWorldSpaceInertia(QCom(), InvI());
		SetAngularAcceleration(WorldInvI * InTorque);
	}

	template <typename T, int d>
	void TPBDRigidParticle<T, d>::AddTorque(const TVector<T, d>& InTorque, bool bInvalidate)
	{
		const FRotation3 RCoM = FParticleUtilitiesGT::GetCoMWorldRotation(this);
		const FMatrix33 WorldInvI = Utilities::ComputeWorldSpaceInertia(RCoM, InvI());
		SetAngularAcceleration(AngularAcceleration() + WorldInvI * InTorque);
	}

	template class TGeometryParticle<FReal, 3>;

	template class TKinematicGeometryParticle<FReal, 3>;

	template class TPBDRigidParticle<FReal, 3>;

	template class TParticleHandleBase<FReal, 3>;
	template class TGeometryParticleHandleImp<FReal, 3, true>;
	template class TKinematicGeometryParticleHandleImp<FReal, 3, true>;
	template class TPBDRigidParticleHandleImp<FReal, 3, true>;

	template <>
	void Chaos::TGeometryParticle<FReal, 3>::MarkDirty(const EChaosPropertyFlags DirtyBits, bool bInvalidate )
	{
		if (bInvalidate)
		{
			this->MDirtyFlags.MarkDirty(DirtyBits);

			if (Proxy)
			{
				if (FPhysicsSolverBase* PhysicsSolverBase = Proxy->GetSolver<FPhysicsSolverBase>())
				{
					PhysicsSolverBase->AddDirtyProxy(Proxy);
				}
			}
		}
	}

	const FVec3 FGenericParticleHandleImp::ZeroVector = FVec3(0);
	const FVec3f FGenericParticleHandleImp::ZeroVectorf = FVec3f(0);
	const FRotation3 FGenericParticleHandleImp::IdentityRotation = FRotation3(FQuat::Identity);
	const FMatrix33 FGenericParticleHandleImp::ZeroMatrix = FMatrix33(0);
	const TUniquePtr<FBVHParticles> FGenericParticleHandleImp::NullBVHParticles = TUniquePtr<FBVHParticles>();
	const FKinematicTarget FGenericParticleHandleImp::EmptyKinematicTarget;

	template <>
	template <>
	int32 TGeometryParticleHandleImp<FReal, 3, true>::GetPayload<int32>(int32 Idx)
	{
		return Idx;
	}

	template <>
	template <>
	int32 TGeometryParticleHandleImp<FReal, 3, false>::GetPayload<int32>(int32 Idx)
	{
		return Idx;
	}

}
