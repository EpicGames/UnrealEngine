// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/PerParticleDampVelocity.h"
#include "Chaos/Matrix.h"
#include "Chaos/SoftsSolverParticlesRange.h"
#include "GenericPlatform/GenericPlatformMath.h"

#if INTEL_ISPC && (!UE_BUILD_SHIPPING || USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING)
#include "HAL/IConsoleManager.h"
#endif

#if INTEL_ISPC
#include "PerParticleDampVelocity.ispc.generated.h"
#endif

#if INTEL_ISPC
static_assert(sizeof(ispc::FVector3f) == sizeof(Chaos::Softs::FSolverVec3), "sizeof(ispc::FVector3f) != sizeof(Chaos::Softs::FSolverVec3)");

#if !UE_BUILD_SHIPPING || USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING
bool bChaos_DampVelocity_ISPC_Enabled = CHAOS_DAMP_VELOCITY_ISPC_ENABLED_DEFAULT;
static FAutoConsoleVariableRef CVarChaosDampVelocityISPCEnabled(TEXT("p.Chaos.DampVelocity.ISPC"), bChaos_DampVelocity_ISPC_Enabled, TEXT("Whether to use ISPC optimizations in per particle damp velocity calculation"));
#endif
#endif

namespace Chaos::Softs {

	void FPerParticleDampVelocity::SetProperties(
		const FCollectionPropertyConstFacade& PropertyCollection,
		const FSolverVec3& ReferenceLocation,
		const FSolverVec3& ReferenceVelocity,
		const FSolverVec3& ReferenceAngularVelocity)
	{
		constexpr FSolverReal DefaultPropertyCoefficient = 0.f;
		if (!bPropertyIndicesInitialized)
		{
			LocalDampingCoefficientIndex = FLocalDampingCoefficientIndex(PropertyCollection);
			LocalDampingLinearCoefficientIndex = FLocalDampingLinearCoefficientIndex(PropertyCollection);
			LocalDampingAngularCoefficientIndex = FLocalDampingAngularCoefficientIndex(PropertyCollection);
			LocalDampingSpaceIndex = FLocalDampingSpaceIndex(PropertyCollection);

			if (LocalDampingLinearCoefficientIndex != INDEX_NONE && LocalDampingAngularCoefficientIndex != INDEX_NONE)
			{
				LinearCoefficient = GetLocalDampingLinearCoefficient(PropertyCollection);
				AngularCoefficient = GetLocalDampingAngularCoefficient(PropertyCollection);
			}
			else
			{
				const float LocalDampingCoefficient = GetLocalDampingCoefficient(PropertyCollection, DefaultPropertyCoefficient);
				LinearCoefficient = AngularCoefficient = LocalDampingCoefficient;
			}
			bPropertyIndicesInitialized = true;
			return;
		}

		if (LocalDampingLinearCoefficientIndex != INDEX_NONE && LocalDampingAngularCoefficientIndex != INDEX_NONE)
		{
			if (IsLocalDampingLinearCoefficientMutable(PropertyCollection))
			{
				LinearCoefficient = GetLocalDampingLinearCoefficient(PropertyCollection);
			}
			if (IsLocalDampingAngularCoefficientMutable(PropertyCollection))
			{
				AngularCoefficient = GetLocalDampingAngularCoefficient(PropertyCollection);
			}
		}
		else
		{
			if (IsLocalDampingCoefficientMutable(PropertyCollection))
			{
				const float LocalDampingCoefficient = GetLocalDampingCoefficient(PropertyCollection);
				LinearCoefficient = AngularCoefficient = LocalDampingCoefficient;
			}

		}

		if (IsLocalDampingSpaceMutable(PropertyCollection))
		{
			LocalDampingSpace = (EChaosSoftsLocalDampingSpace)(GetLocalDampingSpace(PropertyCollection));
		}

		if (LocalDampingSpace == EChaosSoftsLocalDampingSpace::ReferenceBoneSpace)
		{
			Xcm = ReferenceLocation;
			Vcm = ReferenceVelocity;
			Omega = ReferenceAngularVelocity;
		}
	}

	void FPerParticleDampVelocity::UpdatePositionBasedState(const FSolverParticles& Particles, const int32 Offset, const int32 Range)
	{
		if (LocalDampingSpace == EChaosSoftsLocalDampingSpace::CenterOfMass)
		{
			const FSolverParticlesRange ParticlesRange(const_cast<FSolverParticles*>(&Particles), Offset, Range - Offset);
			return UpdatePositionBasedState(ParticlesRange);
		}
	}

	void FPerParticleDampVelocity::UpdatePositionBasedState(const FSolverParticlesRange& Particles)
	{
		if (LocalDampingSpace == EChaosSoftsLocalDampingSpace::CenterOfMass)
		{
#if INTEL_ISPC
			if (bRealTypeCompatibleWithISPC && bChaos_DampVelocity_ISPC_Enabled)
			{
				ispc::UpdatePositionBasedState(
					(ispc::FVector3f&)Xcm,
					(ispc::FVector3f&)Vcm,
					(ispc::FVector3f&)Omega,
					(const ispc::FVector3f*)Particles.XArray().GetData(),
					(const ispc::FVector3f*)Particles.GetV().GetData(),
					Particles.GetM().GetData(),
					Particles.GetInvM().GetData(),
					0,
					Particles.GetRangeSize());
			}
			else
#endif
			{
				Xcm = FSolverVec3(0.);
				Vcm = FSolverVec3(0.);
				FSolverReal Mcm = (FSolverReal)0.;

				for (int32 Index = 0; Index < Particles.GetRangeSize(); ++Index)
				{
					if (Particles.InvM(Index) == (FSolverReal)0.)
					{
						continue;
					}

					Xcm += Particles.GetX(Index) * Particles.M(Index);
					Vcm += Particles.V(Index) * Particles.M(Index);
					Mcm += Particles.M(Index);
				}

				if (Mcm != (FSolverReal)0.)
				{
					Xcm /= Mcm;
					Vcm /= Mcm;
				}

				FSolverVec3 L = FSolverVec3(0.);
				FSolverMatrix33 I(0.);
				for (int32 Index = 0; Index < Particles.GetRangeSize(); ++Index)
				{
					if (Particles.InvM(Index) == (FSolverReal)0.)
					{
						continue;
					}

					const FSolverVec3 V = Particles.GetX(Index) - Xcm;
					L += FSolverVec3::CrossProduct(V, Particles.M(Index) * Particles.V(Index));
					const FSolverMatrix33 M(0, V[2], -V[1], -V[2], 0, V[0], V[1], -V[0], 0);
					I += M.GetTransposed() * M * Particles.M(Index);
				}

				const FSolverReal Det = I.Determinant();
				Omega = Det < (FSolverReal)UE_SMALL_NUMBER || !FMath::IsFinite(Det) ?
					FSolverVec3(0.) :
#if COMPILE_WITHOUT_UNREAL_SUPPORT
					FSolverRigidTransform3(I).InverseTransformVector(L);
#else
					I.InverseTransformVector(L); // Calls FMatrix::InverseFast(), which tests against SMALL_NUMBER
#endif
			}
		}
	}

}  // End namespace Chaos::Softs
