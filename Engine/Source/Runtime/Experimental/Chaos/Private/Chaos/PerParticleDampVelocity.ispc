// Copyright Epic Games, Inc. All Rights Reserved.

#define EXPLICIT_VECTOR4 1
#define EXPLICIT_MATRIX 1

#include "Math/Vector.isph"
#include "Chaos/Matrix33.isph"

static inline uniform bool IsFinite(uniform float A)
{
	return (intbits(A) & 0x7F800000U) != 0x7F800000U;
}

export void UpdatePositionBasedState(uniform FVector3f& MXcm,
									uniform FVector3f& MVcm,
									uniform FVector3f& MOmega,
									const uniform FVector3f X[],
									const uniform FVector3f V[],
									const uniform float M[],
									const uniform float InvM[],
									const uniform int Offset,
									const uniform int32 Range)
{
	varying FVector3f Xcm = FloatZeroVector;
	varying FVector3f Vcm = FloatZeroVector;
	varying float Mcm = 0.;

	foreach(Index = Offset ... Range)
	{
		const varying float IM = InvM[Index];

		if(all(!IM))
		{
			continue;
		}

		if(!IM)
		{
			continue;
		}

		const varying FVector3f MX = VectorLoad(&X[extract(Index, 0)]);
		const varying FVector3f MV = VectorLoad(&V[extract(Index, 0)]);
		const varying float MM = M[Index];

		Xcm = Xcm + (MX * MM);
		Vcm = Vcm + (MV * MM);
		Mcm += MM;
	}

	MXcm = VectorReduceAdd(Xcm);
	MVcm = VectorReduceAdd(Vcm);
	uniform float MMcm = reduce_add(Mcm);

	if (MMcm != FLOAT_ZERO)
	{
		MXcm = MXcm / MMcm;
		MVcm = MVcm / MMcm;
	}

	varying FVector3f VL = FloatZeroVector;
	varying FMatrix33f VI = Matrix33FloatZero;

	foreach(Index = Offset ... Range)
	{
		const varying float IM = InvM[Index];

		if(all(!IM))
		{
			continue;
		}

		if(!IM)
		{
			continue;
		}

		const varying FVector3f MX = VectorLoad(&X[extract(Index, 0)]);
		const varying FVector3f MV = VectorLoad(&V[extract(Index, 0)]);
		const varying float MM = M[Index];

		const varying FVector3f VX = MX - MXcm;
		VL = VL + VectorCross(VX, MM * MV);

		const varying FMatrix33f Mat = SetMatrix33(0, VX.V[2], -VX.V[1], -VX.V[2], 0, VX.V[0], VX.V[1], -VX.V[0], 0);
		VI = AddAB(VI, Multiply(MultiplyABt(Mat, Mat), MM));
	}

	const uniform FVector3f L = VectorReduceAdd(VL);
	const uniform FMatrix33f I = MatrixReduceAdd(VI);

	const uniform FVector3f InverseTransform = MatrixInverseTransformVector(I, L);
	const uniform float Det = MatrixDeterminant(I);

	MOmega = VectorSelect(Det < FLOAT_SMALL_NUMBER || !IsFinite(Det), FloatZeroVector, InverseTransform); // Calls FMatrix::InverseFast(), which tests against SMALL_NUMBER
}
