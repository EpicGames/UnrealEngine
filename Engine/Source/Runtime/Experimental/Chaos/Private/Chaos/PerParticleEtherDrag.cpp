// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/PerParticleEtherDrag.h"
#include "HAL/IConsoleManager.h"

namespace Chaos
{

FRealSingle LinearEtherDragOverride = -1.f;
FAutoConsoleVariableRef CVarLinearEtherDragOverride(TEXT("p.LinearEtherDragOverride"), LinearEtherDragOverride, TEXT("Set an override linear ether drag value. -1.f to disable"));

FRealSingle AngularEtherDragOverride = -1.f;
FAutoConsoleVariableRef CVarAngularEtherDragOverride(TEXT("p.AngularEtherDragOverride"), AngularEtherDragOverride, TEXT("Set an override angular ether drag value. -1.f to disable"));

}
