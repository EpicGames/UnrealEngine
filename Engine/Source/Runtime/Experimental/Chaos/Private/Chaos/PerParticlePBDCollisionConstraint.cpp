// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/PerParticlePBDCollisionConstraint.h"
#include "Chaos/TaperedCapsule.h"
#include "ChaosStats.h"
#include "HAL/IConsoleManager.h"
#if INTEL_ISPC
#include "PerParticlePBDCollisionConstraint.ispc.generated.h"
#include "PerParticlePBDCollisionConstraintISPCDataVerification.h"

#if !UE_BUILD_SHIPPING || USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING 
bool bChaos_PerParticleCollision_ISPC_Enabled = CHAOS_PER_PARTICLE_COLLISION_ISPC_ENABLED_DEFAULT;
static FAutoConsoleVariableRef CVarChaosPerParticleCollisionISPCEnabled(TEXT("p.Chaos.PerParticleCollision.ISPC"), bChaos_PerParticleCollision_ISPC_Enabled, TEXT("Whether to use ISPC optimizations in per particle collisions"));
#endif
#endif

static int32 Chaos_PerParticleCollision_ISPC_ParallelBatchSize = 128;
#if !UE_BUILD_SHIPPING
FAutoConsoleVariableRef CVarChaosPerParticleCollisionISPCParallelBatchSize(TEXT("p.Chaos.PerParticleCollision.ISPC.ParallelBatchSize"), Chaos_PerParticleCollision_ISPC_ParallelBatchSize, TEXT("Parallel batch size for ISPC"));
#endif

namespace Chaos::Softs 
{
namespace Private
{
	// Also called in PBDSoftBodyCollisionConstraint
	void ReflectOneSidedCollision(const FSolverVec3& P, const FSolverVec3& OneSidedPlaneNormal, const FSolverVec3& SplitOrigin, FSolverReal& Penetration, FSolverVec3& ImplicitNormal)
	{
		check(Penetration > (FSolverReal)0.f);

		FSolverVec3 PNew = P + Penetration * ImplicitNormal;
		const FSolverReal SplitAxisProj = (PNew - SplitOrigin).Dot(OneSidedPlaneNormal);
		if (SplitAxisProj >= 0.)
		{
			return;
		}

		PNew -= 2. * SplitAxisProj * OneSidedPlaneNormal;
		ImplicitNormal = PNew - P;
		Penetration = ImplicitNormal.Length();
		ImplicitNormal = Penetration > UE_SMALL_NUMBER ? ImplicitNormal / Penetration : FSolverVec3::ZeroVector;
	}
}

// Helper function to call PhiWithNormal and return data to ISPC
extern "C" void GetPhiWithNormal(const uint8* CollisionParticles, const FSolverReal* InV, FSolverReal* Normal, FSolverReal* Phi, const int32 i, const int32 ProgramCount, const int32 Mask)
{
	const FSolverCollisionParticles& C = *(const FSolverCollisionParticles*)CollisionParticles;
	
	for (int32 Index = 0; Index < ProgramCount; ++Index)
	{
		if (Mask & (1 << Index))
		{
			FSolverVec3 V;

			// aos_to_soa3
			V.X = InV[Index];
			V.Y = InV[Index + ProgramCount];
			V.Z = InV[Index + 2 * ProgramCount];

			FVec3 ImplicitNormal;
			Phi[Index] = (FSolverReal)C.GetGeometry(i)->PhiWithNormal(FVec3(V), ImplicitNormal);
			FSolverVec3 Norm(ImplicitNormal);

			// aos_to_soa3
			Normal[Index] = Norm.X;
			Normal[Index + ProgramCount] = Norm.Y;
			Normal[Index + 2 * ProgramCount] = Norm.Z;
		}
	}
}

void FPerParticlePBDCollisionConstraint::ApplyHelperISPC(FSolverParticles& InParticles, const FSolverReal Dt, const int32 Offset, const int32 Range) const
{
	check(bRealTypeCompatibleWithISPC);
	check(bFastPositionBasedFriction);

	const uint32 DynamicGroupId = MDynamicGroupIds[Offset];
	const FSolverReal PerGroupFriction = MPerGroupFriction[DynamicGroupId];
	const FSolverReal PerGroupThickness = MPerGroupThickness[DynamicGroupId];

	const int32 NumBatches = FMath::CeilToInt((FSolverReal)(Range - Offset) / (FSolverReal)Chaos_PerParticleCollision_ISPC_ParallelBatchSize);

	if (PerGroupFriction > UE_KINDA_SMALL_NUMBER)  // Fast friction
	{
		PhysicsParallelFor(NumBatches, [this, &InParticles, Dt, Offset, Range, DynamicGroupId, PerGroupFriction, PerGroupThickness](int32 BatchNumber)
			{
				const int32 BatchBegin = Offset + (Chaos_PerParticleCollision_ISPC_ParallelBatchSize * BatchNumber);
		const int32 BatchEnd = FMath::Min(Range, BatchBegin + Chaos_PerParticleCollision_ISPC_ParallelBatchSize);

#if INTEL_ISPC
		MCollisionParticlesActiveView.RangeFor(
			[this, &InParticles, Dt, BatchBegin, BatchEnd, DynamicGroupId, PerGroupFriction, PerGroupThickness](FSolverCollisionParticles& CollisionParticles, int32 CollisionOffset, int32 CollisionRange)
			{
				ispc::ApplyPerParticleCollisionFastFriction(
					(ispc::FVector4f*)InParticles.GetPAndInvM().GetData(),
					(const ispc::FVector3f*)InParticles.XArray().GetData(),
					(const ispc::FVector3f*)CollisionParticles.AllV().GetData(),
					(const ispc::FVector3f*)CollisionParticles.XArray().GetData(),
					(const ispc::FVector3f*)CollisionParticles.AllW().GetData(),
					(const ispc::FVector4f*)CollisionParticles.AllR().GetData(),
					DynamicGroupId,
					MKinematicGroupIds.GetData(),
					PerGroupFriction,
					PerGroupThickness,
					(const uint8*)&CollisionParticles,
					(const uint8*)CollisionParticles.GetAllGeometry().GetData(),
					sizeof(FImplicitObject),
					FImplicitObject::GetOffsetOfType(),
					FImplicitObject::GetOffsetOfMargin(),
					Dt,
					CollisionOffset,
					CollisionRange,
					BatchBegin,
					BatchEnd);
			});
#endif  // #if INTEL_ISPC
			});
	}
	else  // No friction
	{
		PhysicsParallelFor(NumBatches, [this, &InParticles, Dt, Offset, Range, DynamicGroupId, PerGroupFriction, PerGroupThickness](int32 BatchNumber)
			{
				const int32 BatchBegin = Offset + (Chaos_PerParticleCollision_ISPC_ParallelBatchSize * BatchNumber);
		const int32 BatchEnd = FMath::Min(Range, BatchBegin + Chaos_PerParticleCollision_ISPC_ParallelBatchSize);

#if INTEL_ISPC
		MCollisionParticlesActiveView.RangeFor(
			[this, &InParticles, Dt, BatchBegin, BatchEnd, DynamicGroupId, PerGroupThickness](FSolverCollisionParticles& CollisionParticles, int32 CollisionOffset, int32 CollisionRange)
			{
				ispc::ApplyPerParticleCollisionNoFriction(
					(ispc::FVector4f*)InParticles.GetPAndInvM().GetData(),
					(const ispc::FVector3f*)InParticles.XArray().GetData(),
					(const ispc::FVector3f*)CollisionParticles.AllV().GetData(),
					(const ispc::FVector3f*)CollisionParticles.XArray().GetData(),
					(const ispc::FVector3f*)CollisionParticles.AllW().GetData(),
					(const ispc::FVector4f*)CollisionParticles.AllR().GetData(),
					DynamicGroupId,
					MKinematicGroupIds.GetData(),
					PerGroupThickness,
					(const uint8*)&CollisionParticles,
					(const uint8*)CollisionParticles.GetAllGeometry().GetData(),
					sizeof(FImplicitObject),
					FImplicitObject::GetOffsetOfType(),
					FImplicitObject::GetOffsetOfMargin(),
					Dt,
					CollisionOffset,
					CollisionRange,
					BatchBegin,
					BatchEnd);
			});
#endif  // #if INTEL_ISPC
			});

	}
}

template<bool bLockAndWriteContacts>
void FPerParticlePBDCollisionConstraint::ApplyHelper(FSolverParticles& Particles, const FSolverReal Dt, const int32 Offset, const int32 Range) const
{
	const uint32 DynamicGroupId = MDynamicGroupIds[Offset];  // Particle group Id, must be the same across the entire range
	const FSolverReal PerGroupFriction = MPerGroupFriction[DynamicGroupId];
	const FSolverReal PerGroupThickness = MPerGroupThickness[DynamicGroupId];

	if (PerGroupFriction > (FSolverReal)UE_KINDA_SMALL_NUMBER)
	{
		PhysicsParallelFor(Range - Offset, [this, &Particles, Dt, Offset, DynamicGroupId, PerGroupFriction, PerGroupThickness](int32 i)
			{
				const int32 Index = Offset + i;

				if (Particles.InvM(Index) == (FSolverReal)0.)
				{
					return;  // Continue
				}

				MCollisionParticlesActiveView.SequentialFor([this, &Particles, &Dt, &Index, DynamicGroupId, PerGroupFriction, PerGroupThickness](FSolverCollisionParticles& CollisionParticles, int32 i)
					{
						const uint32 KinematicGroupId = MKinematicGroupIds[i];  // Collision group Id

						if (KinematicGroupId != (uint32)INDEX_NONE && DynamicGroupId != KinematicGroupId)
						{
							return; // Bail out if the collision groups doesn't match the particle group id, or use INDEX_NONE (= global collision that affects all particle)
						}
						const FImplicitObjectPtr& Geometry = CollisionParticles.GetGeometry(i);
						const FSolverRigidTransform3 Frame(CollisionParticles.GetX(i), CollisionParticles.GetR(i));
						const FVec3 RigidSpacePosition(Frame.InverseTransformPosition(Particles.P(Index)));  // PhiWithNormal requires FReal based arguments
						FVec3 ImplicitNormal;                                                                // since implicits don't use FSolverReal
						const FSolverReal Phi = (FSolverReal)Geometry->PhiWithNormal(RigidSpacePosition, ImplicitNormal);
						FSolverVec3 Normal(ImplicitNormal);
						FSolverReal Penetration = PerGroupThickness - Phi; // This is related to the Normal impulse

						if (Penetration > (FSolverReal)0.)
						{
							// Split capsules always push out in the OneSidedPlaneNormal direction.
							if (const FTaperedCapsule* const Capsule = Geometry->GetObject<FTaperedCapsule>())
							{
								if (Capsule->IsOneSided())
								{
									Private::ReflectOneSidedCollision(FSolverVec3(RigidSpacePosition), Capsule->GetOneSidedPlaneNormalf(), Capsule->GetOriginf(), Penetration, Normal);
								}
							}
							const FSolverVec3 NormalWorld = Frame.TransformVector(Normal);

							if (bLockAndWriteContacts)
							{
								check(Mutex);
								checkSlow(Contacts);
								checkSlow(Normals);
								checkSlow(Phis);
								FScopeLock Lock(Mutex);
								Contacts->Emplace(Particles.P(Index));
								Normals->Emplace(NormalWorld);
								Phis->Emplace(Phi);
							}

							Particles.P(Index) += Penetration * NormalWorld;

							if (bFastPositionBasedFriction)
							{
								FSolverVec3 VectorToPoint = Particles.GetP(Index) - CollisionParticles.GetX(i);
								const FSolverVec3 RelativeDisplacement = (Particles.GetP(Index) - Particles.GetX(Index)) - (CollisionParticles.V(i) + FSolverVec3::CrossProduct(CollisionParticles.W(i), VectorToPoint)) * Dt; // This corresponds to the tangential velocity multiplied by dt (friction will drive this to zero if it is high enough)
								const FSolverVec3 RelativeDisplacementTangent = RelativeDisplacement - FSolverVec3::DotProduct(RelativeDisplacement, NormalWorld) * NormalWorld; // Project displacement into the tangential plane
								const FSolverReal RelativeDisplacementTangentLength = RelativeDisplacementTangent.Size();
								if (RelativeDisplacementTangentLength >= UE_SMALL_NUMBER)
								{
									const FSolverReal PositionCorrection = FMath::Min<FSolverReal>(Penetration * PerGroupFriction, RelativeDisplacementTangentLength);
									const FSolverReal CorrectionRatio = PositionCorrection / RelativeDisplacementTangentLength;
									Particles.P(Index) -= CorrectionRatio * RelativeDisplacementTangent;
								}
							}
							else
							{
								// Note, to fix: Only use fast position based friction for now, since adding to TMaps here is not thread safe when calling Apply on multiple threads (will cause crash)
								FVelocityConstraint Constraint;
								FSolverVec3 VectorToPoint = Particles.GetP(Index) - CollisionParticles.GetX(i);
								Constraint.Velocity = CollisionParticles.V(i) + FSolverVec3::CrossProduct(CollisionParticles.W(i), VectorToPoint);
								Constraint.Normal = Frame.TransformVector(Normal);

								MVelocityConstraints.Add(Index, Constraint);
							}
						}
					});
			});
	}
	else
	{
		PhysicsParallelFor(Range - Offset, [this, &Particles, Dt, Offset, DynamicGroupId, PerGroupFriction, PerGroupThickness](int32 i)
			{
				const int32 Index = Offset + i;

				if (Particles.InvM(Index) == 0)
				{
					return;  // Continue
				}

				MCollisionParticlesActiveView.SequentialFor([this, &Particles, &Dt, &Index, DynamicGroupId, PerGroupFriction, PerGroupThickness](FSolverCollisionParticles& CollisionParticles, int32 i)
					{
						const uint32 KinematicGroupId = MKinematicGroupIds[i];  // Collision group Id

						if (KinematicGroupId != (uint32)INDEX_NONE && DynamicGroupId != KinematicGroupId)
						{
							return; // Bail out if the collision groups doesn't match the particle group id, or use INDEX_NONE (= global collision that affects all particle)
						}
						const FImplicitObjectPtr& Geometry = CollisionParticles.GetGeometry(i);
						const FSolverRigidTransform3 Frame(CollisionParticles.GetX(i), CollisionParticles.GetR(i));
						const FVec3 RigidSpacePosition(Frame.InverseTransformPosition(Particles.P(Index)));  // PhiWithNormal requires FReal based arguments
						FVec3 ImplicitNormal;                                                                // since implicits don't use FSolverReal
						const FSolverReal Phi = (FSolverReal)Geometry->PhiWithNormal(RigidSpacePosition, ImplicitNormal);
						FSolverVec3 Normal(ImplicitNormal);
						FSolverReal Penetration = PerGroupThickness - Phi; // This is related to the Normal impulse

						if (Penetration > (FSolverReal)0.)
						{
							// Split capsules always push out in the OneSidedPlaneNormal direction.
							if (const FTaperedCapsule* const Capsule = Geometry->GetObject<FTaperedCapsule>())
							{
								if (Capsule->IsOneSided())
								{
									Private::ReflectOneSidedCollision(FSolverVec3(RigidSpacePosition), Capsule->GetOneSidedPlaneNormalf(), Capsule->GetOriginf(), Penetration, Normal);
								}
							}

							const FSolverVec3 NormalWorld = Frame.TransformVector(Normal);
							if (bLockAndWriteContacts)
							{
								check(Mutex);
								checkSlow(Contacts);
								checkSlow(Normals);
								checkSlow(Phis);
								FScopeLock Lock(Mutex);
								Contacts->Emplace(Particles.P(Index));
								Normals->Emplace(NormalWorld);
								Phis->Emplace(Phi);
							}

							Particles.P(Index) += Penetration * NormalWorld;
						}
					});
			});
	}
}


void FPerParticlePBDCollisionConstraint::ApplyRange(FSolverParticles& Particles, const FSolverReal Dt, const int32 Offset, const int32 Range) const
{
	// NOTE: currently using ISPC with TWeightedLatticeImplicitObject<FLevelSet> is significantly slower than not using ISPC (largely because it has not been fully implemented)
	if (bRealTypeCompatibleWithISPC && bChaos_PerParticleCollision_ISPC_Enabled && bFastPositionBasedFriction)
	{
		ApplyHelperISPC(Particles, Dt, Offset, Range);
	}
	else
	{
		if (Mutex)
		{
			ApplyHelper<true>(Particles, Dt, Offset, Range);
		}
		else
		{
			ApplyHelper<false>(Particles, Dt, Offset, Range);
		}
	}
}
}  // End namespace Chaos::Softs
