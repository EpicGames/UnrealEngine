// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/PerParticlePBDCollisionConstraint.isph"

extern "C" unmasked void GetPhiWithNormal(const uniform uint8 *uniform CollisionParticles, const uniform float *uniform V, uniform float *uniform Normal, uniform float *uniform Phi, const uniform int i, const uniform int ProgramCount, const uniform int Mask);


// Legacy cloth with friction
export void ApplyPerParticleCollisionFastFriction(uniform FVector4f ParticlesPandInvM[],
													const uniform FVector3f ParticlesX[],
													const uniform FVector3f CollisionV[],
													const uniform FVector3f CollisionX[],
													const uniform FVector3f CollisionW[],
													const uniform FVector4f CollisionR[],
													const uniform uint32 DynamicGroupId,
													const uniform uint32 KinematicGroupIds[],
													const uniform float PerGroupFriction,
													const uniform float PerGroupThickness,
													const uniform uint8 *uniform CollisionParticles,
													const uniform uint8 *uniform Geometry,
													const uniform int SizeofFImplicitObject,
													const uniform int OffsetofGeometryType,
													const uniform int OffsetOfMargin,
													const uniform float Dt,
													const uniform int InnerOffset,
													const uniform int InnerRange,
													const uniform int OuterOffset,
													const uniform int OuterRange)
{
	uniform int BailAt = OuterRange;
	uniform bool bBail = false;

	// Pre-check for bail out.
	foreach(Index = OuterOffset ... BailAt)
	{
		for(uniform int i = InnerOffset; i < InnerRange; i++)
		{
			const uniform uint32 KinematicGroupId = KinematicGroupIds[i];  // Collision group Id

			if (KinematicGroupId != (uint32)INDEX_NONE && DynamicGroupId != KinematicGroupId)
			{
				foreach_active(j)
				{
					if(!bBail)
					{
						BailAt = extract(Index, j);
						bBail = true;
					}
				}
			}
		}
	}

	foreach(Index = OuterOffset ... BailAt)
	{
		const FVector4f PandInvM = VectorLoad(&ParticlesPandInvM[extract(Index, 0)]);
		varying FVector3f P;
		varying float IM;
		UnzipPandInvM(PandInvM, P, IM);

		if(all(IM == 0))
		{
			continue;
		}

		for(uniform int i = InnerOffset; i < InnerRange; i++) // SequentialFor
		{
			// InverseTransformPosition. Can skip Scale calculation because it is 1 for xyz.
			const FVector3f TranslatedVec = P - CollisionX[i];
			const FVector4f InverseR = QuatInverse(CollisionR[i]);
			const FVector3f PhiWithNormalInputFloat = VectorQuaternionRotateVector(InverseR, TranslatedVec);
			FVector3f Normal;
			float Penetration; 

			const uniform SerializedPtr *uniform GeometryPtr = (const uniform SerializedPtr *uniform)Geometry;
			const uniform uint8 *uniform Ptr = GeometryPtr[i].Ptr;
			const uniform uint8 *uniform TypedPtr = Ptr + SizeofFImplicitObject;
			const uniform uint8 Type = Ptr[OffsetofGeometryType] & 0x3F;

			if(!GetSimplePenetrationWithNormal(PhiWithNormalInputFloat, Type, Ptr, TypedPtr, SizeofFImplicitObject, OffsetofGeometryType, OffsetOfMargin, PerGroupThickness, Penetration, Normal))
			{
				uniform int IMMask;
				if(IM != 0)
				{
					IMMask = lanemask();
				}
				float Phi;
				GetPhiWithNormal(CollisionParticles, (const uniform float *uniform)&PhiWithNormalInputFloat, (uniform float *uniform)&Normal, (uniform float *uniform)&Phi, i, programCount, IMMask);
				Penetration = PerGroupThickness - Phi;
			}

			if(Penetration > 0.0f)
			{
				// TransformVector. Scale is 1 so that part can be removed.
				const FVector3f NormalWorld = VectorQuaternionRotateVector(CollisionR[i], Normal);
				P = P + (Penetration * NormalWorld);
				
				const FVector3f X = VectorLoad(&ParticlesX[extract(Index, 0)]);
				ApplyFriction(P, X, NormalWorld, Penetration * PerGroupFriction, Dt, CollisionX[i], CollisionV[i], CollisionW[i]);
			}
		}

		if(IM != 0)
		{
			VectorStore(&ParticlesPandInvM[extract(Index, 0)], SetVector4(P, IM));
		}
	}
}

// Legacy cloth no friction
export void ApplyPerParticleCollisionNoFriction(uniform FVector4f ParticlesPandInvM[],
												const uniform FVector3f ParticlesX[],
												const uniform FVector3f CollisionV[],
												const uniform FVector3f CollisionX[],
												const uniform FVector3f CollisionW[],
												const uniform FVector4f CollisionR[],
												const uniform uint32 DynamicGroupId,
												const uniform uint32 KinematicGroupIds[],
												const uniform float PerGroupThickness,
												const uniform uint8 *uniform CollisionParticles,
												const uniform uint8 *uniform Geometry,
												const uniform int SizeofFImplicitObject,
												const uniform int OffsetofGeometryType,
												const uniform int OffsetOfMargin,
												const uniform float Dt,
												const uniform int InnerOffset,
												const uniform int InnerRange,
												const uniform int OuterOffset,
												const uniform int OuterRange)
{
	uniform int BailAt = OuterRange;
	uniform bool bBail = false;

	// Pre-check for bail out.
	foreach(Index = OuterOffset ... BailAt)
	{
		for(uniform int i = InnerOffset; i < InnerRange; i++)
		{
			const uniform uint32 KinematicGroupId = KinematicGroupIds[i];  // Collision group Id

			if (KinematicGroupId != (uint32)INDEX_NONE && DynamicGroupId != KinematicGroupId)
			{
				foreach_active(j)
				{
					if(!bBail)
					{
						BailAt = extract(Index, j);
						bBail = true;
					}
				}
			}
		}
	}

	foreach(Index = OuterOffset ... BailAt)
	{
		const FVector4f PandInvM = VectorLoad(&ParticlesPandInvM[extract(Index, 0)]);
		varying FVector3f P;
		varying float IM;
		UnzipPandInvM(PandInvM, P, IM);

		if(all(IM == 0))
		{
			continue;
		}

		for(uniform int i = InnerOffset; i < InnerRange; i++) // SequentialFor
		{
			// InverseTransformPosition. Can skip Scale calculation because it is 1 for xyz.
			const FVector3f TranslatedVec = P - CollisionX[i];
			const FVector4f InverseR = QuatInverse(CollisionR[i]);
			const FVector3f PhiWithNormalInputFloat = VectorQuaternionRotateVector(InverseR, TranslatedVec);
			FVector3f Normal;
			float Penetration;

			const uniform SerializedPtr *uniform GeometryPtr = (const uniform SerializedPtr *uniform)Geometry;
			const uniform uint8 *uniform Ptr = GeometryPtr[i].Ptr;
			const uniform uint8 *uniform TypedPtr = Ptr + SizeofFImplicitObject;
			const uniform uint8 Type = Ptr[OffsetofGeometryType] & 0x3F;
			
			if(!GetSimplePenetrationWithNormal(PhiWithNormalInputFloat, Type, Ptr, TypedPtr, SizeofFImplicitObject, OffsetofGeometryType, OffsetOfMargin, PerGroupThickness, Penetration, Normal))
			{
				uniform int IMMask;
				if(IM != 0)
				{
					IMMask = lanemask();
				}
				
				float Phi;
				GetPhiWithNormal(CollisionParticles, (const uniform float *uniform)&PhiWithNormalInputFloat, (uniform float *uniform)&Normal, (uniform float *uniform)&Phi, i, programCount, IMMask);
				Penetration = PerGroupThickness - Phi;
			}

			if(Penetration > 0.0f)
			{
				// TransformVector. Scale is 1 so that part can be removed.
				const FVector3f NormalWorld = VectorQuaternionRotateVector(CollisionR[i], Normal);
				P = P + (Penetration * NormalWorld);
			}
		}

		if(IM != 0)
		{
			VectorStore(&ParticlesPandInvM[extract(Index, 0)], SetVector4(P, IM));
		}
	}
}
