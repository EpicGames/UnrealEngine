// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef __PERPARTICLEPBDCOLLISIONCONSTRAINT_ISPH__
#define __PERPARTICLEPBDCOLLISIONCONSTRAINT_ISPH__

#define EXPLICIT_VECTOR4 1

#include "Math/Vector.isph"
#include "Math/Transform.isph"
#include "Chaos/PBDSofts.isph"

enum ImplicitObjectType
{
	//Note: add entries in order to avoid serialization issues (but before IsInstanced)
	Sphere = 0,
	Box,
	Plane,
	Capsule,
	Transformed,
	Union,
	LevelSet,
	Unknown,
	Convex,
	TaperedCylinder,
	Cylinder,
	TriangleMesh,
	HeightField,
	DEPRECATED_Scaled,	//needed for serialization of existing data
	Triangle,
	UnionClustered,
	TaperedCapsule,
	MLLevelSet,
	SkinnedTriangleMesh,
	ExtrudedTaperedCapsule,
	WeightedLatticeLevelSetType = 38, //== LevelSet | IsWeightedLattice (cannot use expressions like this in ISPC)


	//Add entries above this line for serialization
	IsWeightedLattice = 1 << 5,
	IsInstanced = 1 << 6,
	IsScaled = 1 << 7
};

struct SerializedPtr
{
	uint8 *Ptr;
};

struct UniquePtr
{
	SerializedPtr *Data;
};

struct TArray
{
	uint8 *Data;
	uint32 ArrayNum;
	uint32 ArrayMax;
};

struct PlanesS
{
	unsigned int8 FirstHalfEdgeIndex;
	unsigned int8 NumHalfEdges;
};

struct PlanesM
{
	int16 FirstHalfEdgeIndex;
	int16 NumHalfEdges;
};

struct PlanesL
{
	int32 FirstHalfEdgeIndex;
	int32 NumHalfEdges;
};

struct HalfEdgesS
{
	unsigned int8 PlaneIndex;
	unsigned int8 VertexIndex;
	unsigned int8 TwinHalfEdgeIndex;
};

struct HalfEdgesM
{
	int16 PlaneIndex;
	int16 VertexIndex;
	int16 TwinHalfEdgeIndex;
};

struct HalfEdgesL
{
	int32 PlaneIndex;
	int32 VertexIndex;
	int32 TwinHalfEdgeIndex;
};

struct VerticesS
{
	unsigned int8 FirstHalfEdgeIndex;
};

struct VerticesM
{
	int16 FirstHalfEdgeIndex;
};

struct VerticesL
{
	int32 FirstHalfEdgeIndex;
};

struct Segment
{
	FVector3f Point;
	FVector3f Axis;
	float Length;
};

struct FPlaneConcrete3f
{
	FVector3f MX;
	FVector3f MNormal;
};

struct FConvexStructureDataImp
{
	TArray Planes;
	TArray HalfEdges;
	TArray Vertices;
	TArray Edges;
	// TArray VertexPlanes; // this code never uses this, and we never assume to know the sizeof(FConvexStructureDataImp)
};

enum FConvexStructureDataIndexType
{
	EIndexType_None = 0,
	EIndexType_Small,
	EIndexType_Medium,
	EIndexType_Large
};

struct FConvexStructureData
{
	FConvexStructureDataImp *Data;
	int8 IndexType;
};

struct FConvex
{
	TArray Planes;
	TArray Vertices;
	FVector3f LocalBoundingBoxMin;
	FVector3f LocalBoundingBoxMax;
	FConvexStructureData StructureData;
	// float Volume; // this code never uses this, and we never assume to know the sizeof(FConvex)
	// FVector CenterOfMass; // this code never uses this, and we never assume to know the sizeof(FConvex)
};

struct FTaperedCapsule
{
	FVector3f Origin;
	FVector3f Axis;
	FVector3f OneSidedPlaneNormal;
	float Height;
	float Radius1;
	float Radius2;
	bool bIsOneSided;
	// FAABB3 LocalBoundingBox; // this code never uses this, and we never assume to know the sizeof(FTaperedCapsule)
};

struct FTaperedCylinder
{
	FPlaneConcrete3f MPlane1;
	FPlaneConcrete3f MPlane2;
	float Height;
	float Radius1;
	float Radius2;
	// FAABB3 MLocalBoundingBox; // this code never uses this, and we never assume to know the sizeof(FTaperedCylinder)
};

struct FExtrudedTaperedCapsule
{
	FVector3f Origin;
	FVector3f MajorAxis;
	FVector3f MinorAxis;
	float HalfLength;
	float HalfWidth;
	float Radius1;
	float Radius2;
	// FAABB3 LocalBoundingBox; // this code never uses this, and we never assume to know the sizeof(FExtrudedTaperedCapsule)
};

static inline uniform bool IsComplexBatchCollider(const uniform uint8 CollisionType)
{
	return CollisionType == WeightedLatticeLevelSetType || CollisionType == MLLevelSet;
}

static inline uniform bool IsSimpleCollider(const uniform uint8 CollisionType)
{
	if(IsComplexBatchCollider(CollisionType))
	{
		return false;
	}
	if(CollisionType == SkinnedTriangleMesh)
	{
		return false;
	}
	return true;
}

static inline float SafeNormalize(FVector3f &Direction, const uniform FVector3f& DefaultIfZero)
{
	const float SizeSqr = VectorSizeSquared(Direction);
	const float Size = sqrt(SizeSqr);
	Direction = VectorSelect((SizeSqr < KINDA_SMALL_NUMBER), DefaultIfZero, Direction / Size);
	return (Size < KINDA_SMALL_NUMBER) ? ZERO : Size;
}

static inline float SafeNormalize(FVector3f &Direction)
{
	return SafeNormalize(Direction, FloatForwardVector);
}

static inline bool VectorNormalize(FVector3f &V)
{
	const float SquareSum = VectorSizeSquared(V);
	if(SquareSum > SMALL_NUMBER)
	{
		const float Scale = InvSqrt(SquareSum);
		V = V * Scale;
		return true;
	}
	return false;
}

static inline void ReflectOneSidedCollision(const FVector3f& P, const uniform FVector3f& OneSidedPlaneNormal, const uniform FVector3f& SplitOrigin, float& Penetration, FVector3f& ImplicitNormal)
{
	FVector3f PNew = P + Penetration * ImplicitNormal;
	const float SplitAxisProj = VectorDot(PNew - SplitOrigin, OneSidedPlaneNormal);
	if (SplitAxisProj >= 0.)
	{
		return;
	}

	PNew = PNew - OneSidedPlaneNormal * (2. * SplitAxisProj);
	ImplicitNormal = PNew - P;
	Penetration = SafeNormalize(ImplicitNormal, FloatZeroVector);
}

template<typename ThicknessType>
static inline void PenetrationWithNormalSphere(const uniform uint8 *uniform TypedObjectPtr, const FVector3f &PhiWithNormalInput, const ThicknessType Thickness,  FVector3f &Normal, float &Penetration, const uniform float Radius)
{
	const uniform FVector3f Center = *((const uniform FVector3f *uniform)&TypedObjectPtr[0]);
	Normal = PhiWithNormalInput - Center;
	Penetration = Thickness - (SafeNormalize(Normal) - Radius);
}

static inline void PhiWithNormalPlane(const uniform FPlaneConcrete3f *varying InPlane, const FVector3f &PhiWithNormalInput, FVector3f &Normal, float &Phi)
{
	const varying FVector3f MNormal = VectorGather(&InPlane->MNormal);
	const varying FVector3f MX = VectorGather(&InPlane->MX);

	Normal = MNormal;
	Phi = VectorDot(PhiWithNormalInput - MX, MNormal);
}

static inline float SignedDistancePlane(const uniform FPlaneConcrete3f *uniform InPlane, const FVector3f &x)
{
	const uniform FPlaneConcrete3f P = *InPlane;
	return VectorDot(x - P.MX, P.MNormal);
}

template<typename ThicknessType>
static inline void PenetrationWithNormalCapsule(const uniform uint8 *uniform TypedObjectPtr, const FVector3f &PhiWithNormalInput, const ThicknessType Thickness, FVector3f &Normal, float &Penetration, const uniform float Radius)
{
	const uniform Segment MSegment = *((const uniform Segment *uniform)&TypedObjectPtr[0]);

	const float Dot = clamp(VectorDot(PhiWithNormalInput - MSegment.Point, MSegment.Axis), (uniform float)(0), MSegment.Length);
	const FVector3f ProjectedPoint = MSegment.Axis * Dot + MSegment.Point;
	Normal = PhiWithNormalInput - ProjectedPoint;
	Penetration = Thickness - (SafeNormalize(Normal) - Radius);
}

template<typename ThicknessType>
static inline void PenetrationWithNormalTaperedCapsule(const uniform uint8* uniform TypedObjectPtr, const FVector3f& PhiWithNormalInput, const ThicknessType Thickness, FVector3f& Normal, float& Penetration)
{
	const uniform FTaperedCapsule CapsuleObj = *((const uniform FTaperedCapsule *uniform)&TypedObjectPtr[0]);

	const float DistanceAlongAxis = clamp(VectorDot(PhiWithNormalInput - CapsuleObj.Origin, CapsuleObj.Axis), (uniform float)(0), CapsuleObj.Height);
	const FVector3f ProjectedPoint = CapsuleObj.Axis * DistanceAlongAxis + CapsuleObj.Origin;
	const float Alpha = DistanceAlongAxis / CapsuleObj.Height;
	const float Radius = select(CapsuleObj.Height > FLOAT_SMALL_NUMBER, CapsuleObj.Radius1 * ((uniform float)(1.0) - Alpha) + CapsuleObj.Radius2 * Alpha, max(CapsuleObj.Radius1, CapsuleObj.Radius2));
	Normal = PhiWithNormalInput - ProjectedPoint;
	Penetration = Thickness - (SafeNormalize(Normal) - Radius);

	if(Penetration >= 0.f && CapsuleObj.bIsOneSided)
	{
		ReflectOneSidedCollision(PhiWithNormalInput, CapsuleObj.OneSidedPlaneNormal, CapsuleObj.Origin, Penetration, Normal);
	}
}

template<typename ThicknessType>
static inline void PenetrationWithNormalExtrudedTaperedCapsule(const uniform uint8* uniform TypedObjectPtr, const FVector3f& PhiWithNormalInput, const ThicknessType Thickness, FVector3f& Normal, float& Penetration)
{
	const uniform FExtrudedTaperedCapsule CapsuleObj = *((const uniform FExtrudedTaperedCapsule *uniform)&TypedObjectPtr[0]);

	const FVector3f OriginToX = PhiWithNormalInput - CapsuleObj.Origin;
	const float DistanceAlongMajorAxis = clamp(VectorDot(OriginToX, CapsuleObj.MajorAxis), -CapsuleObj.HalfLength, CapsuleObj.HalfLength);
	const float DistanceAlongMinorAxis = clamp(VectorDot(OriginToX, CapsuleObj.MinorAxis), -CapsuleObj.HalfWidth, CapsuleObj.HalfWidth);
	const FVector3f ProjectedPoint = CapsuleObj.MajorAxis * DistanceAlongMajorAxis + CapsuleObj.MinorAxis * DistanceAlongMinorAxis + CapsuleObj.Origin;
	const float Alpha = .5 * (DistanceAlongMajorAxis / CapsuleObj.HalfLength) + .5;
	const float Radius = select(CapsuleObj.HalfLength > FLOAT_SMALL_NUMBER, CapsuleObj.Radius2 * ((uniform float)(1.0) - Alpha) + CapsuleObj.Radius1 * Alpha, max(CapsuleObj.Radius1, CapsuleObj.Radius2));
	Normal = PhiWithNormalInput - ProjectedPoint;
	Penetration = Thickness - (SafeNormalize(Normal) - Radius);
}

template<typename ThicknessType>
static inline void PenetrationWithNormalTaperedCylinder(const uniform uint8 *uniform TypedObjectPtr, const FVector3f &PhiWithNormalInput, const ThicknessType Thickness, FVector3f &Normal, float &Penetration, const uniform int SizeofFImplicitObject)
{
	const uniform FTaperedCylinder CylinderObj = *((const uniform FTaperedCylinder *uniform)&TypedObjectPtr[0]);

	const float Distance1 = VectorDot(PhiWithNormalInput - CylinderObj.MPlane1.MX, CylinderObj.MPlane1.MNormal);
	// Used to be Distance2 = CylinderObj.MPlane2.PhiWithNormal(x, Normal2); but that would trigger 
	const float Distance2 = CylinderObj.Height - Distance1;          // the ensure on Distance2 being slightly larger than MHeight in some border cases
	const FVector3f SideVector = (PhiWithNormalInput - (CylinderObj.MPlane1.MNormal * Distance1 + CylinderObj.MPlane1.MX));
	const float Alpha = Distance1 / CylinderObj.Height;
	const float Radius = CylinderObj.Radius1 * ((uniform float)(1.0) - Alpha) + CylinderObj.Radius2 * Alpha;
	const float SideDistance = VectorSize(SideVector) - Radius;
	const float TopDistance = Distance1 < Distance2 ? Distance1 : Distance2;

	float Phi = FLT_MAX;

	if (Distance1 < SMALL_NUMBER)
	{
		const FVector3f v = PhiWithNormalInput - (CylinderObj.MPlane1.MNormal * Distance1 + CylinderObj.MPlane1.MX);
		if (VectorSize(v) > CylinderObj.Radius1)
		{
			const FVector3f Corner = VectorGetSafeNormal(v) * CylinderObj.Radius1 + CylinderObj.MPlane1.MX;
			const FVector3f CornerVector = PhiWithNormalInput - Corner;
			Normal = VectorGetSafeNormal(CornerVector);
			Phi = VectorSize(CornerVector);
		}
		else
		{
			Normal = CylinderObj.MPlane1.MNormal * (uniform float)(-1.0);
			Phi = -Distance1;
		}
	}
	else if (Distance2 < SMALL_NUMBER)
	{
		const FVector3f v = PhiWithNormalInput - (CylinderObj.MPlane2.MNormal * Distance2 + CylinderObj.MPlane2.MX);
		if (VectorSize(v) > CylinderObj.Radius2)
		{
			const FVector3f Corner = VectorGetSafeNormal(v) * CylinderObj.Radius2 + CylinderObj.MPlane2.MX;
			const FVector3f CornerVector = PhiWithNormalInput - Corner;
			Normal = VectorGetSafeNormal(CornerVector);
			Phi = VectorSize(CornerVector);
		}
		else
		{
			Normal = CylinderObj.MPlane2.MNormal * (uniform float)(-1.0);
			Phi = -Distance2;
		}
	}
	else if (SideDistance < (uniform float)(0.0) && TopDistance < -SideDistance)
	{
		Normal = Distance1 < Distance2 ? CylinderObj.MPlane1.MNormal * (uniform float)(-1.0) : CylinderObj.MPlane2.MNormal * (uniform float)(-1.0);
		Phi = -TopDistance;
	}
	else
	{
		Normal = VectorGetSafeNormal(SideVector);
		Phi = SideDistance;
	}

	Penetration = Thickness - Phi;
}

template<typename ThicknessType>
static inline void PenetrationWithNormalUnion(const uniform uint8 *uniform TypedPtr, const FVector3f &PhiWithNormalInput, const ThicknessType Thickness, FVector3f &Normal, float &Penetration, const uniform int SizeofFImplicitObject, const uniform int OffsetofGeometryType, const uniform int OffsetOfMargin)
{
	// This is expecting a MObjects = [TaperedCylinder, Sphere, Sphere]
	assert( ((const uniform TArray *uniform)TypedPtr)->ArrayNum == 3 );

	const uniform UniquePtr *uniform MObjects = (const uniform UniquePtr *uniform)TypedPtr;
	Penetration = -FLT_MAX;
	bool NeedsNormalize = false;
	
	uniform uint8 *uniform Object = MObjects->Data[0].Ptr;
	uniform uint8 *uniform TypedObjectPtr = Object + SizeofFImplicitObject;

	FVector3f NextNormal;
	float NextPenetration;

	assert( (Object[OffsetofGeometryType] & 0x3F) == TaperedCylinder );
	PenetrationWithNormalTaperedCylinder(TypedObjectPtr, PhiWithNormalInput, Thickness, NextNormal, NextPenetration, SizeofFImplicitObject);

	if (NextPenetration > Penetration)
	{
		Penetration = NextPenetration;
		Normal = NextNormal;
		NeedsNormalize = false;
	}
	else if (Penetration == NextPenetration)
	{
		Normal = Normal + NextNormal;
		NeedsNormalize = true;
	}

	Object = MObjects->Data[1].Ptr;
	uniform float Margin = *((const uniform float *uniform)(Object + OffsetOfMargin));
	TypedObjectPtr = Object + SizeofFImplicitObject;

	assert( (Object[OffsetofGeometryType] & 0x3F) == Sphere );
	PenetrationWithNormalSphere(TypedObjectPtr, PhiWithNormalInput, Thickness, NextNormal, NextPenetration, Margin);
	
	if (NextPenetration > Penetration)
	{
		Penetration = NextPenetration;
		Normal = NextNormal;
		NeedsNormalize = false;
	}
	else if (Penetration == NextPenetration)
	{
		Normal = Normal + NextNormal;
		NeedsNormalize = true;
	}

	Object = MObjects->Data[2].Ptr;
	Margin = *((const uniform float *uniform)(Object + OffsetOfMargin));
	TypedObjectPtr = Object + SizeofFImplicitObject;

	assert( (Object[OffsetofGeometryType] & 0x3F) == Sphere );
	PenetrationWithNormalSphere(TypedObjectPtr, PhiWithNormalInput, Thickness, NextNormal, NextPenetration, Margin);
	
	if (NextPenetration > Penetration)
	{
		Penetration = NextPenetration;
		Normal = NextNormal;
		NeedsNormalize = false;
	}
	else if (Penetration == NextPenetration)
	{
		Normal = Normal + NextNormal;
		NeedsNormalize = true;
	}

	if(NeedsNormalize)
	{
		VectorNormalize(Normal);
	}
}

static inline FVector3f FindClosestPointOnLineSegment(const FVector3f& P0, const FVector3f& P1, const FVector3f& P)
{
	const FVector3f P10 = P1 - P0;
	const FVector3f PP0 = P - P0;
	const float Proj = VectorDot(P10, PP0);
	const float Denom2 = VectorSizeSquared(P10);
	const float NormalProj = Proj / Denom2;
	const FVector3f P2 = P0 + NormalProj * P10;
	return VectorSelect(Proj < (uniform float)(0.0f) || Denom2 < KINDA_SMALL_NUMBER, P0, VectorSelect(NormalProj > (uniform float)(1.0f), P1, P2));
}

static inline FVector3f FindClosestPointOnTriangle(const FVector3f& ClosestPointOnPlane, const FVector3f& P0, const FVector3f& P1, const FVector3f& P2, const FVector3f& P)
{
	// ComputeBarycentricInPlane
	const FVector3f P10 = P1 - P0;
	const FVector3f P20 = P2 - P0;
	const FVector3f PP0 = P - P0;
	const float Size10 = VectorSizeSquared(P10);
	const float Size20 = VectorSizeSquared(P20);
	const float ProjSides = VectorDot(P10, P20);
	const float ProjP1 = VectorDot(PP0, P10);
	const float ProjP2 = VectorDot(PP0, P20);
	const float Denom = Size10 * Size20 - ProjSides * ProjSides;
	const float BaryX = (Size20 * ProjP1 - ProjSides * ProjP2) / Denom;
	const float BaryY = (Size10 * ProjP2 - ProjSides * ProjP1) / Denom;

	const bool bClosestPoint = (BaryX >= -FLOAT_KINDA_SMALL_NUMBER && BaryX <= 1 + FLOAT_KINDA_SMALL_NUMBER && BaryY >= -FLOAT_KINDA_SMALL_NUMBER && BaryY <= 1 + FLOAT_KINDA_SMALL_NUMBER && (BaryX + BaryY) <= (1 + FLOAT_KINDA_SMALL_NUMBER));

	const FVector3f P10Closest = FindClosestPointOnLineSegment(P0, P1, P);
	const FVector3f P20Closest = FindClosestPointOnLineSegment(P0, P2, P);
	const FVector3f P21Closest = FindClosestPointOnLineSegment(P1, P2, P);

	const float P10Dist2 = VectorSizeSquared(P - P10Closest);
	const float P20Dist2 = VectorSizeSquared(P - P20Closest);
	const float P21Dist2 = VectorSizeSquared(P - P21Closest);

	return VectorSelect(bClosestPoint, ClosestPointOnPlane, VectorSelect(P10Dist2 < P20Dist2, VectorSelect(P10Dist2 < P21Dist2, P10Closest, P21Closest), VectorSelect(P20Dist2 < P21Dist2, P20Closest, P21Closest)));
}

static inline int32 NumPlaneVertices(const uniform FConvexStructureData *uniform StructureData, const varying int32 PlaneIndex)
{
	const uniform int8 IndexType = StructureData->IndexType;

	if(IndexType == EIndexType_Small)
	{
		const uniform PlanesS *uniform Planes = (const uniform PlanesS *uniform)StructureData->Data->Planes.Data;

		#pragma ignore warning(perf)
		return (int32)Planes[PlaneIndex].NumHalfEdges;
	}
	else if(IndexType == EIndexType_Medium)
	{
		const uniform PlanesM *uniform Planes = (const uniform PlanesM *uniform)StructureData->Data->Planes.Data;

		#pragma ignore warning(perf)
		return (int32)Planes[PlaneIndex].NumHalfEdges;
	}
	else if(IndexType == EIndexType_Large)
	{
		const uniform PlanesL *uniform Planes = (const uniform PlanesL *uniform)StructureData->Data->Planes.Data;

		#pragma ignore warning(perf)
		return Planes[PlaneIndex].NumHalfEdges;
	}
	else
	{
		return 0;
	}
}

static inline int32 GetPlaneVertex(const uniform FConvexStructureData *uniform StructureData, const varying int32 PlaneIndex, const uniform int32 PlaneEdgeIndex)
{
	const uniform int8 IndexType = StructureData->IndexType;

	if(IndexType == EIndexType_Small)
	{
		const uniform PlanesS *uniform Planes = (const uniform PlanesS *uniform)StructureData->Data->Planes.Data;
		const uniform HalfEdgesS *uniform HalfEdges = (const uniform HalfEdgesS *uniform)StructureData->Data->HalfEdges.Data;

		#pragma ignore warning(perf)
		const int32 HalfEdgeIndex = (int32)Planes[PlaneIndex].FirstHalfEdgeIndex + PlaneEdgeIndex;

		#pragma ignore warning(perf)
		return (int32)HalfEdges[HalfEdgeIndex].VertexIndex;
	}
	else if(IndexType == EIndexType_Medium)
	{
		const uniform PlanesM *uniform Planes = (const uniform PlanesM *uniform)StructureData->Data->Planes.Data;
		const uniform HalfEdgesM *uniform HalfEdges = (const uniform HalfEdgesM *uniform)StructureData->Data->HalfEdges.Data;

		#pragma ignore warning(perf)
		const int32 HalfEdgeIndex = (int32)Planes[PlaneIndex].FirstHalfEdgeIndex + PlaneEdgeIndex;

		#pragma ignore warning(perf)
		return (int32)HalfEdges[HalfEdgeIndex].VertexIndex;
	}
	else if(IndexType == EIndexType_Large)
	{
		const uniform PlanesL *uniform Planes = (const uniform PlanesL *uniform)StructureData->Data->Planes.Data;
		const uniform HalfEdgesL *uniform HalfEdges = (const uniform HalfEdgesL *uniform)StructureData->Data->HalfEdges.Data;

		#pragma ignore warning(perf)
		const int32 HalfEdgeIndex = Planes[PlaneIndex].FirstHalfEdgeIndex + PlaneEdgeIndex;

		#pragma ignore warning(perf)
		return HalfEdges[HalfEdgeIndex].VertexIndex;
	}
	else
	{
		return 0;
	}
}

template<typename ThicknessType>
static inline void PenetrationWithNormalConvex(const uniform uint8 *uniform TypedObjectPtr, const FVector3f &PhiWithNormalInput, const ThicknessType Thickness, FVector3f &Normal, float &Penetration)
{
	const uniform FConvex *uniform Object = (const uniform FConvex *uniform)TypedObjectPtr;
	const uniform FPlaneConcrete3f *uniform Planes = (const uniform FPlaneConcrete3f *uniform)&Object->Planes.Data[0];
	const uniform int32 NumPlanes = Object->Planes.ArrayNum;

	if (NumPlanes == 0)
	{
		Penetration = -FLT_MAX;
		return;
	}

	float MaxPhi = -FLT_MAX;
	int32 MaxPlane = 0;

	for (uniform int32 Idx = 0; Idx < NumPlanes; ++Idx)
	{
		const float InnerPhi = SignedDistancePlane(&Planes[Idx], PhiWithNormalInput);
		if (InnerPhi > MaxPhi)
		{
			MaxPhi = InnerPhi;
			MaxPlane = Idx;
		}
	}
	
	float Phi = FLT_MAX;
	PhiWithNormalPlane(&Planes[MaxPlane], PhiWithNormalInput, Normal, Phi);
	if (Phi <= 0)
	{
		Penetration = Thickness - Phi;
		return;
	}

	// If x is outside the convex mesh, we should find for the nearest point to triangles on the plane
	const int32 PlaneVerticesNum = NumPlaneVertices(&Object->StructureData, MaxPlane);
	const FVector3f XOnPlane = PhiWithNormalInput - Phi * Normal;

	if(PlaneVerticesNum > 2 && IsVectorEqual(XOnPlane, PhiWithNormalInput))
	{
		Penetration = Thickness - Phi;
		return;
	}

	float ClosestDistance = FLT_MAX;
	FVector3f ClosestPoint = FloatZeroVector;
	for (uniform int32 Index = 0; Index < PlaneVerticesNum - 2; Index++)
	{
		const uniform FVector3f *uniform Vertices = (const uniform FVector3f *uniform)Object->Vertices.Data;
		const uniform FVector3f *varying APtr = &Vertices[GetPlaneVertex(&Object->StructureData, MaxPlane, 0)];
		const uniform FVector3f *varying BPtr = &Vertices[GetPlaneVertex(&Object->StructureData, MaxPlane, Index + 1)];
		const uniform FVector3f *varying CPtr = &Vertices[GetPlaneVertex(&Object->StructureData, MaxPlane, Index + 2)];

		const FVector3f A = VectorGather(APtr);
		const FVector3f B = VectorGather(BPtr);
		const FVector3f C = VectorGather(CPtr);

		const FVector3f Point = FindClosestPointOnTriangle(XOnPlane, A, B, C, PhiWithNormalInput);

		const float Distance = VectorSize(Point - XOnPlane);
		if (Distance < ClosestDistance)
		{
			ClosestDistance = Distance;
			ClosestPoint = Point;
		}
	}

	const FVector3f Difference = PhiWithNormalInput - ClosestPoint;
	Phi = VectorSize(Difference);
	if (Phi > FLOAT_SMALL_NUMBER)
	{
		Normal = (Difference) / Phi;
	}
	Penetration = Thickness - Phi;
}

template<typename ThicknessType>
static inline bool GetSimplePenetrationWithNormal(const FVector3f& PhiWithNormalInputFloat, const uniform uint8 Type, const uniform uint8 *uniform Ptr, const uniform uint8 *uniform TypedPtr, const uniform int SizeofFImplicitObject, const uniform int OffsetofGeometryType, const uniform int OffsetOfMargin, const ThicknessType Thickness, float& Penetration, FVector3f& Normal)
{
	switch(Type)
	{
	case Sphere:
		{
			const uniform float Margin = *((const uniform float *uniform)(Ptr + OffsetOfMargin));
			PenetrationWithNormalSphere(TypedPtr, PhiWithNormalInputFloat, Thickness, Normal, Penetration, Margin);
			return true;
		}
	case Capsule:
		{
			const uniform float Margin = *((const uniform float *uniform)(Ptr + OffsetOfMargin));
			PenetrationWithNormalCapsule(TypedPtr, PhiWithNormalInputFloat, Thickness, Normal, Penetration, Margin);
			return true;
		}
	case Union:
		{
			PenetrationWithNormalUnion(TypedPtr, PhiWithNormalInputFloat, Thickness, Normal, Penetration, SizeofFImplicitObject, OffsetofGeometryType, OffsetOfMargin);
			return true;
		}
	case TaperedCapsule:
		{
			PenetrationWithNormalTaperedCapsule(TypedPtr, PhiWithNormalInputFloat, Thickness, Normal, Penetration);
			return true;
		}
	case Convex:
		{
			PenetrationWithNormalConvex(TypedPtr, PhiWithNormalInputFloat, Thickness, Normal, Penetration);
			return true;
		}
	case ExtrudedTaperedCapsule:
		{
			PenetrationWithNormalExtrudedTaperedCapsule(TypedPtr, PhiWithNormalInputFloat, Thickness, Normal, Penetration);
			return true;
		}

	default:
		return false;
	}
	return false;
}

template<typename ColliderVecType>
static inline void ApplyFriction(FVector3f& P, const FVector3f& X, const FVector3f& NormalWorld, const float MaxFrictionCorrection, const uniform float Dt, const ColliderVecType& CollisionX, const ColliderVecType& CollisionV, const ColliderVecType& CollisionW, FVector3f& ColliderVelocityAtPoint)
{
	const FVector3f VectorToPoint = P - CollisionX;
	ColliderVelocityAtPoint = CollisionV + VectorCross(CollisionW, VectorToPoint);

	const FVector3f RelativeDisplacement = (P - X) - ColliderVelocityAtPoint * Dt; // This corresponds to the tangential velocity multiplied by dt (friction will drive this to zero if it is high enough)
	const FVector3f RelativeDisplacementTangent = RelativeDisplacement - VectorDot(RelativeDisplacement, NormalWorld) * NormalWorld; // Project displacement into the tangential plane
	const float RelativeDisplacementTangentLength = VectorSize(RelativeDisplacementTangent);
	const float PositionCorrection = min(MaxFrictionCorrection, RelativeDisplacementTangentLength);
	const float CorrectionRatio = select(RelativeDisplacementTangentLength < FLOAT_SMALL_NUMBER, 0.0f, PositionCorrection / RelativeDisplacementTangentLength);
	P = P - (CorrectionRatio * RelativeDisplacementTangent);
}

template<typename ColliderVecType>
static inline void ApplyFriction(FVector3f& P, const FVector3f& X, const FVector3f& NormalWorld, const float MaxFrictionCorrection, const uniform float Dt, const ColliderVecType& CollisionX, const ColliderVecType& CollisionV, const ColliderVecType& CollisionW)
{
	FVector3f ColliderVelocityAtPoint;
	ApplyFriction(P, X, NormalWorld, MaxFrictionCorrection, Dt, CollisionX, CollisionV, CollisionW, ColliderVelocityAtPoint);
}

#endif
