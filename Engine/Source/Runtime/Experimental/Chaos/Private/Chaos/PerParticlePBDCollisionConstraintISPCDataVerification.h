// Copyright Epic Games, Inc. All Rights Reserved.

#if INTEL_ISPC
#include "PerParticlePBDCollisionConstraintISPCDataVerification.ispc.generated.h"
#include "Chaos/PBDSoftsEvolutionFwd.h"
#include "Chaos/Capsule.h"
#include "Chaos/Convex.h"
#include "Chaos/ExtrudedTaperedCapsule.h"
#include "Chaos/ImplicitObjectUnion.h"
#include "Chaos/Sphere.h"
#include "Chaos/TaperedCapsule.h"
#include "Chaos/TaperedCylinder.h"

static_assert(sizeof(ispc::FVector3f) == sizeof(Chaos::Softs::FSolverVec3), "sizeof(ispc::FVector3f) != sizeof(Chaos::Softs::FSolverVec3)");
static_assert(sizeof(ispc::FVector4f) == sizeof(Chaos::Softs::FSolverRotation3), "sizeof(ispc::FVector4f) != sizeof(Chaos::Softs::FSolverRotation3)");
static_assert(ispc::ImplicitObjectType::Sphere == Chaos::ImplicitObjectType::Sphere);
static_assert(ispc::ImplicitObjectType::Capsule == Chaos::ImplicitObjectType::Capsule);
static_assert(ispc::ImplicitObjectType::Union == Chaos::ImplicitObjectType::Union);
static_assert(ispc::ImplicitObjectType::TaperedCapsule == Chaos::ImplicitObjectType::TaperedCapsule);
static_assert(ispc::ImplicitObjectType::Convex == Chaos::ImplicitObjectType::Convex);
static_assert(ispc::ImplicitObjectType::IsWeightedLattice == Chaos::ImplicitObjectType::IsWeightedLattice);
static_assert(ispc::ImplicitObjectType::MLLevelSet == Chaos::ImplicitObjectType::MLLevelSet);
static_assert(ispc::ImplicitObjectType::SkinnedTriangleMesh == Chaos::ImplicitObjectType::SkinnedTriangleMesh);
static_assert(ispc::ImplicitObjectType::WeightedLatticeLevelSetType == (Chaos::ImplicitObjectType::IsWeightedLattice | Chaos::ImplicitObjectType::LevelSet));
static_assert(sizeof(ispc::TArray) == sizeof(TArray<int>));
// Sphere
static_assert(sizeof(Chaos::FImplicitObject) == Chaos::FSphere::FISPCDataVerifier::OffsetOfCenter());
static_assert(sizeof(ispc::FVector3f) == Chaos::FSphere::FISPCDataVerifier::SizeOfCenter());
// Capsule
static_assert(sizeof(Chaos::FImplicitObject) == Chaos::FCapsule::FISPCDataVerifier::OffsetOfMSegment());
static_assert(sizeof(ispc::Segment) == Chaos::FCapsule::FISPCDataVerifier::SizeOfMSegment());
// Union (only specific case of MObjects = [FTaperedCylinder, Sphere, Sphere] is used here.
static_assert(sizeof(Chaos::FImplicitObject) == Chaos::FImplicitObjectUnion::FISPCDataVerifier::OffsetOfMObjects());
static_assert(sizeof(ispc::TArray) == Chaos::FImplicitObjectUnion::FISPCDataVerifier::SizeOfMObjects());
// TaperedCylinder
static_assert(sizeof(Chaos::FImplicitObject) + offsetof(ispc::FTaperedCylinder, MPlane1) == Chaos::FTaperedCylinder::FISPCDataVerifier::OffsetOfMPlane1());
static_assert(sizeof(ispc::FTaperedCylinder::MPlane1) == Chaos::FTaperedCylinder::FISPCDataVerifier::SizeOfMPlane1());
static_assert(sizeof(Chaos::FImplicitObject) + offsetof(ispc::FTaperedCylinder, MPlane2) == Chaos::FTaperedCylinder::FISPCDataVerifier::OffsetOfMPlane2());
static_assert(sizeof(ispc::FTaperedCylinder::MPlane2) == Chaos::FTaperedCylinder::FISPCDataVerifier::SizeOfMPlane2());
static_assert(sizeof(Chaos::FImplicitObject) + offsetof(ispc::FTaperedCylinder, Height) == Chaos::FTaperedCylinder::FISPCDataVerifier::OffsetOfMHeight());
static_assert(sizeof(ispc::FTaperedCylinder::Height) == Chaos::FTaperedCylinder::FISPCDataVerifier::SizeOfMHeight());
static_assert(sizeof(Chaos::FImplicitObject) + offsetof(ispc::FTaperedCylinder, Radius1) == Chaos::FTaperedCylinder::FISPCDataVerifier::OffsetOfMRadius1());
static_assert(sizeof(ispc::FTaperedCylinder::Radius1) == Chaos::FTaperedCylinder::FISPCDataVerifier::SizeOfMRadius1());
static_assert(sizeof(Chaos::FImplicitObject) + offsetof(ispc::FTaperedCylinder, Radius2) == Chaos::FTaperedCylinder::FISPCDataVerifier::OffsetOfMRadius2());
static_assert(sizeof(ispc::FTaperedCylinder::Radius2) == Chaos::FTaperedCylinder::FISPCDataVerifier::SizeOfMRadius2());
// TaperedCapsule
static_assert(sizeof(Chaos::FImplicitObject) + offsetof(ispc::FTaperedCapsule, Origin) == Chaos::FTaperedCapsule::FISPCDataVerifier::OffsetOfOrigin());
static_assert(sizeof(ispc::FTaperedCapsule::Origin) == Chaos::FTaperedCapsule::FISPCDataVerifier::SizeOfOrigin());
static_assert(sizeof(Chaos::FImplicitObject) + offsetof(ispc::FTaperedCapsule, Axis) == Chaos::FTaperedCapsule::FISPCDataVerifier::OffsetOfAxis());
static_assert(sizeof(ispc::FTaperedCapsule::Axis) == Chaos::FTaperedCapsule::FISPCDataVerifier::SizeOfAxis());
static_assert(sizeof(Chaos::FImplicitObject) + offsetof(ispc::FTaperedCapsule, OneSidedPlaneNormal) == Chaos::FTaperedCapsule::FISPCDataVerifier::OffsetOfOneSidedPlaneNormal());
static_assert(sizeof(ispc::FTaperedCapsule::OneSidedPlaneNormal) == Chaos::FTaperedCapsule::FISPCDataVerifier::SizeOfOneSidedPlaneNormal());
static_assert(sizeof(Chaos::FImplicitObject) + offsetof(ispc::FTaperedCapsule, Height) == Chaos::FTaperedCapsule::FISPCDataVerifier::OffsetOfHeight());
static_assert(sizeof(ispc::FTaperedCapsule::Height) == Chaos::FTaperedCapsule::FISPCDataVerifier::SizeOfHeight());
static_assert(sizeof(Chaos::FImplicitObject) + offsetof(ispc::FTaperedCapsule, Radius1) == Chaos::FTaperedCapsule::FISPCDataVerifier::OffsetOfRadius1());
static_assert(sizeof(ispc::FTaperedCapsule::Radius1) == Chaos::FTaperedCapsule::FISPCDataVerifier::SizeOfRadius1());
static_assert(sizeof(Chaos::FImplicitObject) + offsetof(ispc::FTaperedCapsule, Radius2) == Chaos::FTaperedCapsule::FISPCDataVerifier::OffsetOfRadius2());
static_assert(sizeof(ispc::FTaperedCapsule::Radius2) == Chaos::FTaperedCapsule::FISPCDataVerifier::SizeOfRadius2());
static_assert(sizeof(Chaos::FImplicitObject) + offsetof(ispc::FTaperedCapsule, bIsOneSided) == Chaos::FTaperedCapsule::FISPCDataVerifier::OffsetOfIsOneSided());
static_assert(sizeof(ispc::FTaperedCapsule::bIsOneSided) == Chaos::FTaperedCapsule::FISPCDataVerifier::SizeOfIsOneSided());
// ExtrudedTaperedCapsule
static_assert(sizeof(Chaos::FImplicitObject) + offsetof(ispc::FExtrudedTaperedCapsule, Origin) == Chaos::FExtrudedTaperedCapsule::FISPCDataVerifier::OffsetOfOrigin());
static_assert(sizeof(ispc::FExtrudedTaperedCapsule::Origin) == Chaos::FExtrudedTaperedCapsule::FISPCDataVerifier::SizeOfOrigin());
static_assert(sizeof(Chaos::FImplicitObject) + offsetof(ispc::FExtrudedTaperedCapsule, MajorAxis) == Chaos::FExtrudedTaperedCapsule::FISPCDataVerifier::OffsetOfMajorAxis());
static_assert(sizeof(ispc::FExtrudedTaperedCapsule::MajorAxis) == Chaos::FExtrudedTaperedCapsule::FISPCDataVerifier::SizeOfMajorAxis());
static_assert(sizeof(Chaos::FImplicitObject) + offsetof(ispc::FExtrudedTaperedCapsule, MinorAxis) == Chaos::FExtrudedTaperedCapsule::FISPCDataVerifier::OffsetOfMinorAxis());
static_assert(sizeof(ispc::FExtrudedTaperedCapsule::MinorAxis) == Chaos::FExtrudedTaperedCapsule::FISPCDataVerifier::SizeOfMinorAxis());
static_assert(sizeof(Chaos::FImplicitObject) + offsetof(ispc::FExtrudedTaperedCapsule, HalfLength) == Chaos::FExtrudedTaperedCapsule::FISPCDataVerifier::OffsetOfHalfLength());
static_assert(sizeof(ispc::FExtrudedTaperedCapsule::HalfLength) == Chaos::FExtrudedTaperedCapsule::FISPCDataVerifier::SizeOfHalfLength());
static_assert(sizeof(Chaos::FImplicitObject) + offsetof(ispc::FExtrudedTaperedCapsule, HalfWidth) == Chaos::FExtrudedTaperedCapsule::FISPCDataVerifier::OffsetOfHalfWidth());
static_assert(sizeof(ispc::FExtrudedTaperedCapsule::HalfWidth) == Chaos::FExtrudedTaperedCapsule::FISPCDataVerifier::SizeOfHalfWidth());
static_assert(sizeof(Chaos::FImplicitObject) + offsetof(ispc::FExtrudedTaperedCapsule, Radius1) == Chaos::FExtrudedTaperedCapsule::FISPCDataVerifier::OffsetOfRadius1());
static_assert(sizeof(ispc::FExtrudedTaperedCapsule::Radius1) == Chaos::FExtrudedTaperedCapsule::FISPCDataVerifier::SizeOfRadius1());
static_assert(sizeof(Chaos::FImplicitObject) + offsetof(ispc::FExtrudedTaperedCapsule, Radius2) == Chaos::FExtrudedTaperedCapsule::FISPCDataVerifier::OffsetOfRadius2());
static_assert(sizeof(ispc::FExtrudedTaperedCapsule::Radius2) == Chaos::FExtrudedTaperedCapsule::FISPCDataVerifier::SizeOfRadius2());

// Convex
static_assert(sizeof(Chaos::FImplicitObject) + offsetof(ispc::FConvex, Planes) == Chaos::FConvex::FISPCDataVerifier::OffsetOfPlanes());
static_assert(sizeof(ispc::FConvex::Planes) == Chaos::FConvex::FISPCDataVerifier::SizeOfPlanes());
static_assert(sizeof(Chaos::FImplicitObject) + offsetof(ispc::FConvex, Vertices) == Chaos::FConvex::FISPCDataVerifier::OffsetOfVertices());
static_assert(sizeof(ispc::FConvex::Vertices) == Chaos::FConvex::FISPCDataVerifier::SizeOfVertices());
static_assert(sizeof(Chaos::FImplicitObject) + offsetof(ispc::FConvex, StructureData) == Chaos::FConvex::FISPCDataVerifier::OffsetOfStructureData());
static_assert(sizeof(ispc::FConvex::StructureData) == Chaos::FConvex::FISPCDataVerifier::SizeOfStructureData());

static_assert(sizeof(ispc::FPlaneConcrete3f) == sizeof(Chaos::FConvex::FPlaneType));
static_assert(sizeof(ispc::FVector3f) == sizeof(Chaos::FConvex::FVec3Type)); // Vertices array

static_assert(offsetof(ispc::FConvexStructureData, Data) == Chaos::FConvexStructureData::FISPCDataVerifier::OffsetOfData());
static_assert(sizeof(ispc::FConvexStructureData::Data) == Chaos::FConvexStructureData::FISPCDataVerifier::SizeOfData()); //-V568
static_assert(offsetof(ispc::FConvexStructureData, IndexType) == Chaos::FConvexStructureData::FISPCDataVerifier::OffsetOfIndexType());
static_assert(sizeof(ispc::FConvexStructureData::IndexType) == Chaos::FConvexStructureData::FISPCDataVerifier::SizeOfIndexType());

static_assert(offsetof(ispc::FConvexStructureDataImp, Planes) == Chaos::FConvexStructureData::FConvexStructureDataSmall::FISPCDataVerifier::OffsetOfPlanes());
static_assert(offsetof(ispc::FConvexStructureDataImp, Planes) == Chaos::FConvexStructureData::FConvexStructureDataMedium::FISPCDataVerifier::OffsetOfPlanes());
static_assert(offsetof(ispc::FConvexStructureDataImp, Planes) == Chaos::FConvexStructureData::FConvexStructureDataLarge::FISPCDataVerifier::OffsetOfPlanes());
static_assert(sizeof(ispc::FConvexStructureDataImp::Planes) == Chaos::FConvexStructureData::FConvexStructureDataSmall::FISPCDataVerifier::SizeOfPlanes());
static_assert(sizeof(ispc::FConvexStructureDataImp::Planes) == Chaos::FConvexStructureData::FConvexStructureDataMedium::FISPCDataVerifier::SizeOfPlanes());
static_assert(sizeof(ispc::FConvexStructureDataImp::Planes) == Chaos::FConvexStructureData::FConvexStructureDataLarge::FISPCDataVerifier::SizeOfPlanes());

static_assert(offsetof(ispc::FConvexStructureDataImp, HalfEdges) == Chaos::FConvexStructureData::FConvexStructureDataSmall::FISPCDataVerifier::OffsetOfHalfEdges());
static_assert(offsetof(ispc::FConvexStructureDataImp, HalfEdges) == Chaos::FConvexStructureData::FConvexStructureDataMedium::FISPCDataVerifier::OffsetOfHalfEdges());
static_assert(offsetof(ispc::FConvexStructureDataImp, HalfEdges) == Chaos::FConvexStructureData::FConvexStructureDataLarge::FISPCDataVerifier::OffsetOfHalfEdges());
static_assert(sizeof(ispc::FConvexStructureDataImp::HalfEdges) == Chaos::FConvexStructureData::FConvexStructureDataSmall::FISPCDataVerifier::SizeOfHalfEdges());
static_assert(sizeof(ispc::FConvexStructureDataImp::HalfEdges) == Chaos::FConvexStructureData::FConvexStructureDataMedium::FISPCDataVerifier::SizeOfHalfEdges());
static_assert(sizeof(ispc::FConvexStructureDataImp::HalfEdges) == Chaos::FConvexStructureData::FConvexStructureDataLarge::FISPCDataVerifier::SizeOfHalfEdges());

static_assert(offsetof(ispc::FConvexStructureDataImp, Vertices) == Chaos::FConvexStructureData::FConvexStructureDataSmall::FISPCDataVerifier::OffsetOfVertices());
static_assert(offsetof(ispc::FConvexStructureDataImp, Vertices) == Chaos::FConvexStructureData::FConvexStructureDataMedium::FISPCDataVerifier::OffsetOfVertices());
static_assert(offsetof(ispc::FConvexStructureDataImp, Vertices) == Chaos::FConvexStructureData::FConvexStructureDataLarge::FISPCDataVerifier::OffsetOfVertices());
static_assert(sizeof(ispc::FConvexStructureDataImp::Vertices) == Chaos::FConvexStructureData::FConvexStructureDataSmall::FISPCDataVerifier::SizeOfVertices());
static_assert(sizeof(ispc::FConvexStructureDataImp::Vertices) == Chaos::FConvexStructureData::FConvexStructureDataMedium::FISPCDataVerifier::SizeOfVertices());
static_assert(sizeof(ispc::FConvexStructureDataImp::Vertices) == Chaos::FConvexStructureData::FConvexStructureDataLarge::FISPCDataVerifier::SizeOfVertices());

static_assert(sizeof(ispc::PlanesS) == sizeof(Chaos::FConvexStructureData::FConvexStructureDataSmall::FPlaneData));
static_assert(sizeof(ispc::PlanesM) == sizeof(Chaos::FConvexStructureData::FConvexStructureDataMedium::FPlaneData));
static_assert(sizeof(ispc::PlanesL) == sizeof(Chaos::FConvexStructureData::FConvexStructureDataLarge::FPlaneData));
static_assert(sizeof(ispc::HalfEdgesS) == sizeof(Chaos::FConvexStructureData::FConvexStructureDataSmall::FHalfEdgeData));
static_assert(sizeof(ispc::HalfEdgesM) == sizeof(Chaos::FConvexStructureData::FConvexStructureDataMedium::FHalfEdgeData));
static_assert(sizeof(ispc::HalfEdgesL) == sizeof(Chaos::FConvexStructureData::FConvexStructureDataLarge::FHalfEdgeData));
static_assert(sizeof(ispc::VerticesS) == sizeof(Chaos::FConvexStructureData::FConvexStructureDataSmall::FVertexData));
static_assert(sizeof(ispc::VerticesM) == sizeof(Chaos::FConvexStructureData::FConvexStructureDataMedium::FVertexData));
static_assert(sizeof(ispc::VerticesL) == sizeof(Chaos::FConvexStructureData::FConvexStructureDataLarge::FVertexData));

#endif
