// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/PerParticlePBDCollisionConstraint.isph"

// We want to export all of the structs above, so defining exported method that uses them.
export uniform int32 ExportHack_PerParticlePBDCollisionConstraintStructs(
	uniform ImplicitObjectType Type,
	uniform TArray* uniform Array,
	uniform PlanesS* uniform _PlanesS,
	uniform PlanesM* uniform _PlanesM,
	uniform PlanesL* uniform _PlanesL,
	uniform HalfEdgesS* uniform _HalfEdgesS,
	uniform HalfEdgesM* uniform _HalfEdgesM,
	uniform HalfEdgesL* uniform _HalfEdgesL,
	uniform VerticesS* uniform _VerticesS,
	uniform VerticesM* uniform _VerticesM,
	uniform VerticesL* uniform _VerticesL,
	uniform Segment* uniform _Segment,
	uniform FPlaneConcrete3f* uniform _Plane3f,
	uniform FConvexStructureDataIndexType IndexType,
	uniform FConvex* uniform _FConvex,
	uniform FTaperedCapsule* uniform _TaperedCapsule,
	uniform FTaperedCylinder* uniform _TaperedCylinder,
	uniform FExtrudedTaperedCapsule* uniform _ExtrudedTaperedCapsule,
	uniform FVector3f* uniform _Vector3f,
	uniform FVector4f* uniform _Vector4f
)
{
	return 0;
}