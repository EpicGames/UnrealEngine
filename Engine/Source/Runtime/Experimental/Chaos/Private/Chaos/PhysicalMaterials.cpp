// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/PhysicalMaterials.h"
#include "HAL/LowLevelMemTracker.h"
#include "Chaos/AsyncInitBodyHelper.h"

namespace Chaos
{
	FChaosPhysicsMaterial* FMaterialHandle::Get() const
	{
		if(InnerHandle.IsValid())
		{
			return FPhysicalMaterialManager::Get().Resolve(InnerHandle);
		}
		return nullptr;
	}

	const FChaosPhysicsMaterial* FConstMaterialHandle::Get() const
	{
		if(InnerHandle.IsValid())
		{
			return FPhysicalMaterialManager::Get().Resolve(InnerHandle);
		}
		return nullptr;
	}

	FChaosPhysicsMaterial* FMaterialHandle::GetInternal(const THandleArray<FChaosPhysicsMaterial>* const SimMaterials) const
	{
		if (SimMaterials != nullptr && InnerHandle.IsValid())
		{
			return SimMaterials->Get(InnerHandle);
		}
		return nullptr;
	}


	FChaosPhysicsMaterialMask* FMaterialMaskHandle::Get() const
	{
		if (InnerHandle.IsValid())
		{
			return FPhysicalMaterialManager::Get().Resolve(InnerHandle);
		}
		return nullptr;
	}

	const FChaosPhysicsMaterialMask* FConstMaterialMaskHandle::Get() const
	{
		if (InnerHandle.IsValid())
		{
			return FPhysicalMaterialManager::Get().Resolve(InnerHandle);
		}
		return nullptr;
	}

	FPhysicalMaterialManager::FPhysicalMaterialManager()
		: Materials(InitialCapacity)
		, MaterialMasks()
	{

	}

	FPhysicalMaterialManager& FPhysicalMaterialManager::Get()
	{
		static FPhysicalMaterialManager Instance;
		return Instance;
	}

	FChaosPhysicsMaterial* FPhysicalMaterialManager::Resolve(FChaosMaterialHandle InHandle) const
	{
		UE_CHAOS_ASYNC_INITBODY_READSCOPELOCK(MaterialsLock);
		return Materials.Get(InHandle);
	}

	const FChaosPhysicsMaterial* FPhysicalMaterialManager::Resolve(FChaosConstMaterialHandle InHandle) const
	{
		UE_CHAOS_ASYNC_INITBODY_READSCOPELOCK(MaterialsLock);
		return Materials.Get(InHandle);
	}

	FChaosPhysicsMaterialMask* FPhysicalMaterialManager::Resolve(FChaosMaterialMaskHandle InHandle) const
	{
		UE_CHAOS_ASYNC_INITBODY_READSCOPELOCK(MaterialMasksLock);
		return MaterialMasks.Get(InHandle);
	}

	const FChaosPhysicsMaterialMask* FPhysicalMaterialManager::Resolve(FChaosConstMaterialMaskHandle InHandle) const
	{
		UE_CHAOS_ASYNC_INITBODY_READSCOPELOCK(MaterialMasksLock);
		return MaterialMasks.Get(InHandle);
	}
	
	void FPhysicalMaterialManager::UpdateMaterial(FMaterialHandle InHandle)
	{
		check(Chaos::CVars::bEnableAsyncInitBody || IsInGameThread());
		OnMaterialUpdated.Broadcast(InHandle);
	}

	void FPhysicalMaterialManager::UpdateMaterialMask(FMaterialMaskHandle InHandle)
	{
		check(Chaos::CVars::bEnableAsyncInitBody || IsInGameThread());
		OnMaterialMaskUpdated.Broadcast(InHandle);
	}

	const Chaos::THandleArray<FChaosPhysicsMaterial>& FPhysicalMaterialManager::GetMasterMaterials_External() const
	{
		return GetPrimaryMaterials_External();
	}

	const Chaos::THandleArray<FChaosPhysicsMaterialMask>& FPhysicalMaterialManager::GetMasterMaterialMasks_External() const
	{
		return GetPrimaryMaterialMasks_External();
	}
	
	const Chaos::THandleArray<FChaosPhysicsMaterial>& FPhysicalMaterialManager::GetPrimaryMaterials_External() const
	{
		return Materials;
	}

	const Chaos::THandleArray<FChaosPhysicsMaterialMask>& FPhysicalMaterialManager::GetPrimaryMaterialMasks_External() const
	{
		return MaterialMasks;
	}

	FMaterialHandle FPhysicalMaterialManager::Create()
	{
		LLM_SCOPE(ELLMTag::ChaosMaterial);

		check(Chaos::CVars::bEnableAsyncInitBody || IsInGameThread());
		FMaterialHandle OutHandle;
		{
			UE_CHAOS_ASYNC_INITBODY_WRITESCOPELOCK(MaterialsLock);
			OutHandle.InnerHandle = Materials.Create();
		}
		OnMaterialCreated.Broadcast(OutHandle);

		return OutHandle;
	}

	FMaterialMaskHandle FPhysicalMaterialManager::CreateMask()
	{
		check(Chaos::CVars::bEnableAsyncInitBody || IsInGameThread());
		FMaterialMaskHandle OutHandle;
		{
			UE_CHAOS_ASYNC_INITBODY_WRITESCOPELOCK(MaterialMasksLock);
			OutHandle.InnerHandle = MaterialMasks.Create();
		}
		OnMaterialMaskCreated.Broadcast(OutHandle);

		return OutHandle;
	}

	void FPhysicalMaterialManager::Destroy(FMaterialHandle InHandle)
	{
		LLM_SCOPE(ELLMTag::ChaosMaterial);

		check(Chaos::CVars::bEnableAsyncInitBody || IsInGameThread());
		if(InHandle.InnerHandle.IsValid())
		{
			OnMaterialDestroyed.Broadcast(InHandle);
			{
				UE_CHAOS_ASYNC_INITBODY_WRITESCOPELOCK(MaterialsLock);
				Materials.Destroy(InHandle.InnerHandle);
			}
		}
	}

	void FPhysicalMaterialManager::Destroy(FMaterialMaskHandle InHandle)
	{
		check(Chaos::CVars::bEnableAsyncInitBody || IsInGameThread());
		if (InHandle.InnerHandle.IsValid())
		{
			OnMaterialMaskDestroyed.Broadcast(InHandle);
			{
				UE_CHAOS_ASYNC_INITBODY_WRITESCOPELOCK(MaterialMasksLock);
				MaterialMasks.Destroy(InHandle.InnerHandle);
			}
		}
	}
}