// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/PhysicsMaterialUtilities.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/ShapeInstance.h"

namespace Chaos
{
	namespace Private
	{
		const FChaosPhysicsMaterial* GetPhysicsMaterial(
			const TGeometryParticleHandle<FReal, 3>* Particle,
			const FShapeInstance* Shape,
			const int32 ShapeFaceIndex,
			const TArrayCollectionArray<TSerializablePtr<FChaosPhysicsMaterial>>* PhysicsMaterials,
			const TArrayCollectionArray<TUniquePtr<FChaosPhysicsMaterial>>* PerParticlePhysicsMaterials,
			const THandleArray<FChaosPhysicsMaterial>* const SimMaterials)
		{
			// Use the per-particle material if it exists
			if (PerParticlePhysicsMaterials != nullptr)
			{
				const FChaosPhysicsMaterial* UniquePhysicsMaterial = Particle->AuxilaryValue(*PerParticlePhysicsMaterials).Get();
				if (UniquePhysicsMaterial != nullptr)
				{
					return UniquePhysicsMaterial;
				}
			}

			// Use the.... other?.... per particle material if it exists. 
			// @todo(chaos): I assume this is to support writable unique materials, but there must be a better 
			// way that adding another pointer to every particle. E.g., a ref count or shared setting on the material.
			if (PhysicsMaterials != nullptr)
			{
				const FChaosPhysicsMaterial* PhysicsMaterial = Particle->AuxilaryValue(*PhysicsMaterials).Get();
				if (PhysicsMaterial != nullptr)
				{
					return PhysicsMaterial;
				}
			}

			// If no particle material, see if the shape has one
			if ((Shape != nullptr) && (SimMaterials != nullptr) && (Shape->NumMaterialsInternal(SimMaterials) > 0))
			{
				// By default we take the first material (all non-mesh shapes will use material 0)
				int32 ShapeMaterialIndex = 0;

				if ((Shape->NumMaterialsInternal(SimMaterials) > 1) && (ShapeFaceIndex != INDEX_NONE))
				{
					// Multi-shape object (tri mesh, heightfield, etc) - use the face index
					ShapeMaterialIndex = Shape->GetGeometry()->GetMaterialIndex(ShapeFaceIndex);
				}

				if (ShapeMaterialIndex < Shape->NumMaterialsInternal(SimMaterials))
				{
					return SimMaterials->Get(Shape->GetMaterialInternal(ShapeMaterialIndex, SimMaterials).InnerHandle);
				}
			}

			// Either no material is assigned, or the geometry used for this particle 
			// does not belong to the particle. This can happen in the case of fracture.
			return nullptr;
		}

		const FChaosPhysicsMaterial* GetFirstPhysicsMaterial(const TGeometryParticleHandle<FReal, 3>* Particle, const TArrayCollectionArray<TSerializablePtr<FChaosPhysicsMaterial>>* PhysicsMaterials, const TArrayCollectionArray<TUniquePtr<FChaosPhysicsMaterial>>* PerParticlePhysicsMaterials, const THandleArray<FChaosPhysicsMaterial>* const SimMaterials)
		{
			const FShapeInstance* Shape = (Particle->ShapeInstances().IsEmpty()) ? nullptr : Particle->ShapeInstances()[0].Get();
			return GetPhysicsMaterial(Particle, Shape, INDEX_NONE, PhysicsMaterials, PerParticlePhysicsMaterials, SimMaterials);
		}
	}
}