// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/PhysicsObject.h"
#include "Chaos/PhysicsObjectInternal.h"
#include "Chaos/Framework/PhysicsProxyBase.h"

namespace Chaos
{
	void FPhysicsObjectDeleter::operator()(FPhysicsObjectHandle p)
	{
		delete p;
	}

	FPhysicsObjectUniquePtr FPhysicsObjectFactory::CreatePhysicsObject(IPhysicsProxyBase* InProxy, int32 InBodyIndex, const FName& InBodyName)
	{
		return FPhysicsObjectUniquePtr{new FPhysicsObject(InProxy, InBodyIndex, InBodyName)};
	}

	bool FPhysicsObject::IsValid() const
	{
		return Proxy != nullptr && !Proxy->GetMarkedDeleted();
	}
}