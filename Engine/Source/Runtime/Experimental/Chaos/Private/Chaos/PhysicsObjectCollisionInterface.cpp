// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/PhysicsObjectCollisionInterface.h"

namespace Chaos
{
	template<EThreadContext Id>
	bool FPhysicsObjectCollisionInterface<Id>::PhysicsObjectOverlap(const FConstPhysicsObjectHandle ObjectA, const FTransform& InTransformA, const FConstPhysicsObjectHandle ObjectB, const FTransform& InTransformB, bool bTraceComplex)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPhysicsObjectCollisionInterface<Id>::PhysicsObjectOverlap);
		return PairwiseShapeOverlapHelper(ObjectA, InTransformA, ObjectB, InTransformB, bTraceComplex, false, FVector::Zero(), [](const FShapeOverlapData&, const FShapeOverlapData&, const FMTDInfo&) { return false; });
	}

	template<EThreadContext Id>
	bool FPhysicsObjectCollisionInterface<Id>::PhysicsObjectOverlapWithMTD(const FConstPhysicsObjectHandle ObjectA, const FTransform& InTransformA, const FConstPhysicsObjectHandle ObjectB, const FTransform& InTransformB, bool bTraceComplex, FMTDInfo& OutMTD)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPhysicsObjectCollisionInterface<Id>::PhysicsObjectOverlapWithMTD);
		OutMTD.Penetration = 0.0;
		return PairwiseShapeOverlapHelper(
			ObjectA,
			InTransformA,
			ObjectB,
			InTransformB,
			bTraceComplex,
			true,
			FVector::Zero(),
			[&OutMTD](const FShapeOverlapData&, const FShapeOverlapData&, const FMTDInfo& MTDInfo)
			{
				if (MTDInfo.Penetration > OutMTD.Penetration)
				{
					OutMTD = MTDInfo;
				}
				return true;
			}
		);
	}

	template<EThreadContext Id>
	bool FPhysicsObjectCollisionInterface<Id>::PhysicsObjectOverlapWithAABB(const FConstPhysicsObjectHandle ObjectA, const FTransform& InTransformA, const FConstPhysicsObjectHandle ObjectB, const FTransform& InTransformB, bool bTraceComplex, const FVector& Tolerance, FBox& OutOverlap)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPhysicsObjectCollisionInterface<Id>::PhysicsObjectOverlapWithAABB);
		OutOverlap = FBox{ EForceInit::ForceInitToZero };
		return PairwiseShapeOverlapHelper(
			ObjectA,
			InTransformA,
			ObjectB,
			InTransformB,
			bTraceComplex,
			false,
			Tolerance,
			[&OutOverlap, &Tolerance](const FShapeOverlapData& ShapeA, const FShapeOverlapData& ShapeB, const FMTDInfo&)
			{
				const FAABB3 Intersection = ShapeA.BoundingBox.GetIntersection(ShapeB.BoundingBox);
				OutOverlap += FBox{ Intersection.Min(), Intersection.Max() };
				return true;
			}
		);
	}

	template<EThreadContext Id>
	bool FPhysicsObjectCollisionInterface<Id>::PhysicsObjectOverlapWithAABBSize(const FConstPhysicsObjectHandle ObjectA, const FTransform& InTransformA, const FConstPhysicsObjectHandle ObjectB, const FTransform& InTransformB, bool bTraceComplex, const FVector& Tolerance, FVector& OutOverlapSize)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPhysicsObjectCollisionInterface<Id>::PhysicsObjectOverlapWithAABBSize);
		OutOverlapSize = FVector::Zero();
		return PairwiseShapeOverlapHelper(
			ObjectA,
			InTransformA,
			ObjectB,
			InTransformB,
			bTraceComplex,
			false,
			Tolerance,
			[&OutOverlapSize](const FShapeOverlapData& ShapeA, const FShapeOverlapData& ShapeB, const FMTDInfo&)
			{
				const FAABB3 Intersection = ShapeA.BoundingBox.GetIntersection(ShapeB.BoundingBox);
				OutOverlapSize += Intersection.Extents();
				return true;
			}
		);
	}

	template<EThreadContext Id>
	bool FPhysicsObjectCollisionInterface<Id>::PhysicsObjectOverlapWithAABBIntersections(const FConstPhysicsObjectHandle ObjectA, const FTransform& InTransformA, const FConstPhysicsObjectHandle ObjectB, const FTransform& InTransformB, bool bTraceComplex, const FVector& Tolerance, TArray<FBox>& OutIntersections)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPhysicsObjectCollisionInterface<Id>::PhysicsObjectOverlapWithAABBIntersections);
		OutIntersections.Reset();
		return PairwiseShapeOverlapHelper(
			ObjectA,
			InTransformA,
			ObjectB,
			InTransformB,
			bTraceComplex,
			false,
			Tolerance,
			[&OutIntersections, &Tolerance](const FShapeOverlapData& ShapeA, const FShapeOverlapData& ShapeB, const FMTDInfo&)
			{
				const FAABB3 Intersection = ShapeA.BoundingBox.GetIntersection(ShapeB.BoundingBox);
				OutIntersections.Emplace(Intersection.Min(), Intersection.Max());
				return true;
			}
		);
	}

	template<EThreadContext Id>
	bool FPhysicsObjectCollisionInterface<Id>::PairwiseShapeOverlapHelper(const FConstPhysicsObjectHandle ObjectA, const FTransform& InTransformA, const FConstPhysicsObjectHandle ObjectB, const FTransform& InTransformB, bool bTraceComplex, bool bComputeMTD, const FVector& Tolerance, const TFunction<bool(const FShapeOverlapData&, const FShapeOverlapData&, const FMTDInfo&)>& Lambda)
	{
		TArray<TThreadShapeInstance<Id>*> ShapesA = Interface.GetAllThreadShapes({ &ObjectA, 1 });
		const FTransform TransformA = FTransform{ Interface.GetR(ObjectA), Interface.GetX(ObjectA) } *InTransformA;
		const FBox BoxA = Interface.GetWorldBounds({ &ObjectA, 1 }).TransformBy(InTransformA);

		TArray<TThreadShapeInstance<Id>*> ShapesB = Interface.GetAllThreadShapes({ &ObjectB, 1 });
		const FTransform TransformB = FTransform{ Interface.GetR(ObjectB), Interface.GetX(ObjectB) } *InTransformB;
		const FBox BoxB = Interface.GetWorldBounds({ &ObjectB, 1 }).TransformBy(InTransformB);

		if (!BoxA.Intersect(BoxB))
		{
			return false;
		}

		bool bFoundOverlap = false;
		for (TThreadShapeInstance<Id>* B : ShapesB)
		{
			if (!B)
			{
				continue;
			}

			const FImplicitObjectRef GeomB = B->GetGeometry();
			if (!GeomB || !GeomB->IsConvex())
			{
				continue;
			}

			const FAABB3 BoxShapeB = GeomB->CalculateTransformedBounds(TransformB).ShrinkSymmetrically(Tolerance);
			// At this point on, this function should be mirror the Overlap_GeomInternal function in PhysInterface_Chaos.cpp.
			// ShapeA is equivalent to InInstance and GeomB is equivalent to InGeom.

			for (TThreadShapeInstance<Id>* A : ShapesA)
			{
				if (!A)
				{
					continue;
				}

				const Filter::FShapeFilterData& FilterData = A->GetShapeFilterData();
				const bool bShapeIsComplex = FilterData.HasFlag(Chaos::EFilterFlags::ComplexCollision);
				const bool bShapeIsSimple = FilterData.HasFlag(Chaos::EFilterFlags::SimpleCollision);
				const bool bShouldTrace = (bTraceComplex && bShapeIsComplex) || (!bTraceComplex && bShapeIsSimple);
				if (!bShouldTrace)
				{
					continue;
				}

				const FAABB3 BoxShapeA = A->GetGeometry()->CalculateTransformedBounds(TransformA).ShrinkSymmetrically(Tolerance);
				if (!BoxShapeA.Intersects(BoxShapeB))
				{
					continue;
				}

				Chaos::FMTDInfo TmpMTDInfo;
				const bool bOverlap = Chaos::Utilities::CastHelper(
					*GeomB,
					TransformB,
					[A, &TransformA, bComputeMTD, &TmpMTDInfo](const auto& Downcast, const auto& FullTransformB)
					{
						return Chaos::OverlapQuery(*A->GetGeometry(), TransformA, Downcast, FullTransformB, 0, bComputeMTD ? &TmpMTDInfo : nullptr);
					}
				);

				if (bOverlap)
				{
					bFoundOverlap = true;

					FShapeOverlapData OverlapDataA = { A, BoxShapeA };
					FShapeOverlapData OverlapDataB = { B, BoxShapeB };

					if (!Lambda(OverlapDataA, OverlapDataB, TmpMTDInfo))
					{
						return true;
					}
				}
			}
		}

		return bFoundOverlap;
	}

	template class FPhysicsObjectCollisionInterface<EThreadContext::External>;
	template class FPhysicsObjectCollisionInterface<EThreadContext::Internal>;
}