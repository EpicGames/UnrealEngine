// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/PhysicsObjectInterface.h"

#include "Chaos/CastingUtilities.h"
#include "Chaos/ChaosUserEntity.h"
#include "Chaos/CollisionFilterData.h"
#include "Chaos/PhysicsObjectCollisionInterface.h"
#include "Chaos/PhysicsObjectInternalInterface.h"
#include "Chaos/PhysicsObjectInternal.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "Math/Transform.h"
#include "Chaos/ChaosEngineInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PhysicsObjectInterface)

namespace PhysicsObjectInterfaceCVars
{
	static float StrainModifier = 200;
	static FAutoConsoleVariableRef CVarStrainModifier(
		TEXT("Chaos.Debug.StrainModifier"),
		StrainModifier,
		TEXT("(Deprecated) When using radial impulse, compute the strain by multiplier the impulse by this factor"),
		ECVF_Default
	);

	static bool RadialImpulseDistributeToChildren = true;
	static FAutoConsoleVariableRef CVarRadialImpulseDistributeToChildren(
		TEXT("Chaos.Debug.RadialImpulseDistributeToChildren"),
		RadialImpulseDistributeToChildren,
		TEXT("When one and applied to a geometry collection cluster, the impulse will be divided equally betweemn all the children"),
		ECVF_Default
	);
}

namespace
{
	Chaos::FPhysicsSolverBase* GetSolverForObjects(TArrayView<const Chaos::FPhysicsObjectHandle> InView)
	{
		return InView.IsEmpty() ? nullptr : Chaos::FPhysicsObjectInterface::GetSolver(InView[0]);
	}

	template<Chaos::EThreadContext Id>
	void SetParticleStateHelper(const Chaos::FPhysicsObjectHandle PhysicsObject, Chaos::EObjectStateType State)
	{
		if (!PhysicsObject)
		{
			return;
		}

		IPhysicsProxyBase* Proxy = PhysicsObject->PhysicsProxy();
		Chaos::TThreadParticle<Id>* Particle = PhysicsObject->GetParticle<Id>();
		if (!Particle || !Proxy)
		{
			return;
		}

		if (Chaos::TThreadRigidParticle<Id>* Rigid = Particle->CastToRigidParticle())
		{
			if constexpr (Id == Chaos::EThreadContext::External)
			{
				if (Proxy->GetType() == EPhysicsProxyType::SingleParticleProxy)
				{
					// Easiest way to maintain the same behavior as what we currently have for the single particle case on the game thread.
					static_cast<Chaos::FSingleParticlePhysicsProxy*>(Proxy)->GetGameThreadAPI().SetObjectState(State);
				}
				else
				{
					Rigid->SetObjectState(State, false, false);

					// In the case of the geometry collection, it won't marshal the state from the game thread to the physics thread
					// so we need to do it for it manually. 
					if (Proxy->GetType() == EPhysicsProxyType::GeometryCollectionType)
					{
						if (Chaos::FPhysicsSolverBase* Solver = Proxy->GetSolverBase())
						{
							Solver->EnqueueCommandImmediate(
								[PhysicsObject, State]() {
									SetParticleStateHelper<Chaos::EThreadContext::Internal>(PhysicsObject, State);
								}
							);
						}
					}
				}
			}
			else
			{
				if (Chaos::FPBDRigidsSolver* Solver = Proxy->GetSolver<Chaos::FPBDRigidsSolver>())
				{
					if (Chaos::FPBDRigidsSolver::FPBDRigidsEvolution* Evolution = Solver->GetEvolution())
					{
						Evolution->SetParticleObjectState(Rigid, State);
					}
				}
			}
		}
	}

	// Apply an impulse based on an origin,radius and falloff parameters
	// return the fall off value ( between 0 and 1 )
	template<typename T>
	float AddRadialImpulseHelper(T ParticleHandle, const FVector& Origin, const float Radius, const float Strength, const enum ERadialImpulseFalloff Falloff, const float VelocityRatio = 1.0f, bool bInvalidate = true, bool bVelChange = false, float MinValue = 0.f, float MaxValue = 1.f)
	{
		using namespace Chaos;

		float FalloffAlpha = 0.f;

		if (ParticleHandle)
		{
			FVec3 ParticleX = ParticleHandle->GetX();
			if (ParticleHandle->Disabled())
			{
				if (FPBDRigidClusteredParticleHandle* ClusteredParticle = ParticleHandle->CastToClustered())
				{
					if (const FPBDRigidClusteredParticleHandle* ClusteredParent = ClusteredParticle->Parent())
					{
						const FTransform ParentTransform(ClusteredParent->GetR(), ClusteredParent->GetX());
						ParticleX = ParentTransform.TransformPosition(ClusteredParticle->ChildToParent().GetTranslation());
					}
				}
			}

			const FVec3 ParticleToOrigin = ParticleX - Origin;
			const float DistanceSquared = (float)ParticleToOrigin.SizeSquared();
			const float RadiusSquared = Radius * Radius;

			if (DistanceSquared < RadiusSquared)
			{
				// by default we are within the radius so strength is maximum
				// equivalent to ERadialImpulseFalloff::RIF_Constant
				FalloffAlpha = 1.0f;

				if (Falloff == ERadialImpulseFalloff::RIF_Linear)
				{
					const float Distance = FMath::Sqrt(DistanceSquared);
					FalloffAlpha = static_cast<float>(1.0f - Distance / Radius);

					if (FalloffAlpha >= 0)
					{
						const float PrevFalloffAlpha = FalloffAlpha;
						// remap the value within MinValue/MaxValue within the radius
						FalloffAlpha = FMath::Lerp(MinValue, MaxValue, FalloffAlpha);
					}
					else
					{
						// outside of the sphere, clamp to 0
						FalloffAlpha = 0.f;
					}
				}

				// if the strength was still strong enough to consider
				if (FalloffAlpha > 0)
				{
					const FVec3 Normal = ParticleToOrigin.GetSafeNormal();
					const FVec3 Impulse = Normal * FalloffAlpha * Strength;
					const FReal InvMass = bVelChange ? 1.0 : ParticleHandle->InvM();
					const FVec3 Velocity = Impulse * InvMass * VelocityRatio;

					const FVec3 CurrentImpulseVelocity = ParticleHandle->LinearImpulseVelocity();

					ParticleHandle->SetLinearImpulseVelocity(CurrentImpulseVelocity + Velocity, bInvalidate);
				}
			}
		}
		return FalloffAlpha;
	}
}

FName FClosestPhysicsObjectResult::HitName() const
{
	if (!PhysicsObject)
	{
		return NAME_None;
	}
	return Chaos::FPhysicsObjectInterface::GetName(PhysicsObject);
}

namespace Chaos
{
	template<EThreadContext Id>
	FPhysicsObjectHandle FReadPhysicsObjectInterface<Id>::GetRootObject(const FConstPhysicsObjectHandle Object)
	{
		if (!Object)
		{
			return nullptr;
		}

		return Object->GetRootObject<Id>();
	}

	template<EThreadContext Id>
	bool FReadPhysicsObjectInterface<Id>::HasChildren(const FConstPhysicsObjectHandle Object)
	{
		return Object ? Object->HasChildren<Id>() : false;
	}

	template<EThreadContext Id>
	FChaosUserDefinedEntity* FReadPhysicsObjectInterface<Id>::GetUserDefinedEntity(const FConstPhysicsObjectHandle Object)
	{
		if constexpr (Id == EThreadContext::External)
		{
			if (!Object)
			{
				return nullptr;
			}

			if (FGeometryParticle* Particle = Object->GetParticle<Id>())
			{
				// Do we have a user entity appended?
				FChaosUserEntityAppend* UserEntityAppend = FChaosUserData::Get<FChaosUserEntityAppend>(Particle->UserData());
				if (UserEntityAppend)
				{
					return UserEntityAppend->UserDefinedEntity;
				}
			}
		}
		else
		{
			ensureMsgf(false, TEXT("User Defined Entities can only be read by the game thread"));
		}
		return nullptr;
	}

	template<EThreadContext Id>
	int32 FReadPhysicsObjectInterface<Id>::GetClusterHierarchyLevel(const FConstPhysicsObjectHandle Object)
	{
		if (!Object)
		{
			return INDEX_NONE;
		}

		// TODO: I think the GC rest collection caches this information. If we're on the GT it might be
		// better for perf to try and grab that information somehow instead of walking the hierarchy.

		int32 CurrentLevel = 0;

		FPhysicsObjectHandle Parent = Object->GetParentObject<Id>();
		while (Parent)
		{
			// Being part of a cluster union doesn't count in terms of its default cluster hierarchy level.
			if (Parent->PhysicsProxy() != Object->PhysicsProxy())
			{
				break;
			}

			Parent = Parent->GetParentObject<Id>();
			++CurrentLevel;
		}

		return CurrentLevel;
	}

	template<EThreadContext Id>
	FTransform FReadPhysicsObjectInterface<Id>::GetTransform(const FConstPhysicsObjectHandle Object)
	{
		return FTransform{ GetR(Object), GetX(Object) };
	}

	template<EThreadContext Id>
	FVector FReadPhysicsObjectInterface<Id>::GetX(const FConstPhysicsObjectHandle Object)
	{
		if (!Object)
		{
			return FVector::Zero();
		}

		if (TThreadParticle<Id>* Particle = Object->GetParticle<Id>())
		{
			return Particle->GetX();
		}

		return FVector::Zero();
	}

	template<EThreadContext Id>
	FVector FReadPhysicsObjectInterface<Id>::GetCoM(const FConstPhysicsObjectHandle Object)
	{
		if (!Object)
		{
			return FVector::Zero();
		}

		if (TThreadParticle<Id>* Particle = Object->GetParticle<Id>())
		{
			if (Chaos::TThreadRigidParticle<Id>* Rigid = Particle->CastToRigidParticle())
			{
				return Rigid->CenterOfMass();
			}
		}

		return FVector::Zero();
	}

	template<EThreadContext Id>
	FVector FReadPhysicsObjectInterface<Id>::GetWorldCoM(const FConstPhysicsObjectHandle Object)
	{
		return GetX(Object) + GetR(Object).RotateVector(GetCoM(Object));
	}

	template<EThreadContext Id>
	FQuat FReadPhysicsObjectInterface<Id>::GetR(const FConstPhysicsObjectHandle Object)
	{
		if (!Object)
		{
			return FQuat::Identity;
		}

		if (TThreadParticle<Id>* Particle = Object->GetParticle<Id>())
		{
			return Particle->GetR();
		}

		return FQuat::Identity;
	}

	template<EThreadContext Id>
	FVector FReadPhysicsObjectInterface<Id>::GetV(const FConstPhysicsObjectHandle Object)
	{
		if (!Object)
		{
			return FVector::Zero();
		}

		if (TThreadParticle<Id>* Particle = Object->GetParticle<Id>())
		{
			if (Chaos::TThreadRigidParticle<Id>* Rigid = Particle->CastToRigidParticle())
			{
				return Rigid->GetV();
			}
		}

		return FVector::Zero();
	}

	template<EThreadContext Id>
	FVector FReadPhysicsObjectInterface<Id>::GetVAtPoint(const FConstPhysicsObjectHandle Object, const FVector& Point)
	{
		if (!Object)
		{
			return FVector::Zero();
		}

		if (TThreadParticle<Id>* Particle = Object->GetParticle<Id>())
		{
			if (TThreadKinematicParticle<Id>* Kinematic = Particle->CastToKinematicParticle())
			{
				const FVector CenterOfMass = GetWorldCoM(Object);
				const FVector Diff = Point - CenterOfMass;
				return Kinematic->GetV() - FVector::CrossProduct(Diff, Kinematic->GetW());
			}
		}

		return FVector::Zero();
	}
	template<EThreadContext Id>
	FVector FReadPhysicsObjectInterface<Id>::GetW(const FConstPhysicsObjectHandle Object)
	{
		if (!Object)
		{
			return FVector::Zero();
		}

		if (TThreadParticle<Id>* Particle = Object->GetParticle<Id>())
		{
			if (Chaos::TThreadRigidParticle<Id>* Rigid = Particle->CastToRigidParticle())
			{
				return Rigid->GetW();
			}
		}

		return FVector::Zero();
	}

	template<EThreadContext Id>
	FSpatialAccelerationIdx FReadPhysicsObjectInterface<Id>::GetSpatialIndex(const FConstPhysicsObjectHandle Object)
	{
		if (!Object)
		{
			return {};
		}

		if (TThreadParticle<Id>* Particle = Object->GetParticle<Id>())
		{
			return Particle->SpatialIdx();
		}

		return {};
	}

	template<EThreadContext Id>
	TThreadParticle<Id>* FReadPhysicsObjectInterface<Id>::GetParticle(const FConstPhysicsObjectHandle Handle)
	{
		if (!Handle)
		{
			return nullptr;
		}
		return Handle->GetParticle<Id>();
	}

	template<EThreadContext Id>
	TThreadKinematicParticle<Id>* FReadPhysicsObjectInterface<Id>::GetKinematicParticle(const FConstPhysicsObjectHandle Handle)
	{
		if (Handle)
		{
			if (TThreadParticle<Id>* Particle = Handle->GetParticle<Id>())
			{
				if (TThreadKinematicParticle<Id>* KinematicParticle = Particle->CastToKinematicParticle())
				{
					return KinematicParticle;
				}
			}
		}

		return nullptr;
	}

	template<EThreadContext Id>
	TThreadRigidParticle<Id>* FReadPhysicsObjectInterface<Id>::GetRigidParticle(const FConstPhysicsObjectHandle Handle)
	{
		if (Handle)
		{
			if (TThreadParticle<Id>* Particle = Handle->GetParticle<Id>())
			{
				if (TThreadRigidParticle<Id>* RigidParticle = Particle->CastToRigidParticle())
				{
					return RigidParticle;
				}
			}
		}

		return nullptr;
	}

	template<EThreadContext Id>
	TArray<TThreadParticle<Id>*> FReadPhysicsObjectInterface<Id>::GetAllParticles(TArrayView<const FConstPhysicsObjectHandle> InObjects)
	{
		TArray<TThreadParticle<Id>*> Particles;
		Particles.Reserve(InObjects.Num());

		for (const FConstPhysicsObjectHandle& Handle : InObjects)
		{
			if (!Handle)
			{
				continue;
			}

			if (TThreadParticle<Id>* Particle = Handle->GetParticle<Id>())
			{
				Particles.Add(Particle);
			}
		}

		return Particles;
	}

	template<EThreadContext Id>
	TArray<TThreadRigidParticle<Id>*> FReadPhysicsObjectInterface<Id>::GetAllRigidParticles(TArrayView<const FConstPhysicsObjectHandle> InObjects, bool bIncludeNulls)
	{
		TArray<TThreadRigidParticle<Id>*> Particles;
		Particles.Reserve(InObjects.Num());

		for (const FConstPhysicsObjectHandle& Handle : InObjects)
		{
			TThreadRigidParticle<Id>* RigidParticle = nullptr;

			if (Handle)
			{
				if (TThreadParticle<Id>* Particle = Handle->GetParticle<Id>())
				{
					RigidParticle = Particle->CastToRigidParticle();
				}
			}

			if (RigidParticle || bIncludeNulls)
			{
				Particles.Add(RigidParticle);
			}
		}

		return Particles;
	}

	template<EThreadContext Id>
	TArray<FPerShapeData*> FReadPhysicsObjectInterface<Id>::GetAllShapes(TArrayView<const FConstPhysicsObjectHandle> InObjects)
	{
		TArray<FPerShapeData*> AllShapes;

		for (const FConstPhysicsObjectHandle Object : InObjects)
		{
			if (!Object)
			{
				continue;
			}

			if (TThreadParticle<Id>* Particle = Object->GetParticle<Id>())
			{
				const Chaos::FShapesArray& ShapesArray = Particle->ShapesArray();
				for (const TUniquePtr<Chaos::FPerShapeData>& Shape : ShapesArray)
				{
					AllShapes.Add(Shape.Get());
				}
			}
		}

		return AllShapes;
	}

	template<EThreadContext Id>
	TArray<TThreadShapeInstance<Id>*> FReadPhysicsObjectInterface<Id>::GetAllThreadShapes(TArrayView<const FConstPhysicsObjectHandle> InObjects)
	{
		TArray<TThreadShapeInstance<Id>*> AllShapes;

		for (const FConstPhysicsObjectHandle Object : InObjects)
		{
			if (!Object)
			{
				continue;
			}

			if (TThreadParticle<Id>* Particle = Object->GetParticle<Id>())
			{
				for (const TUniquePtr<TThreadShapeInstance<Id>>& Shape : Particle->ShapeInstances())
				{
					AllShapes.Add(Shape.Get());
				}
			}
		}

		return AllShapes;
	}

	template<EThreadContext Id>
	FImplicitObjectRef FReadPhysicsObjectInterface<Id>::GetGeometry(const FConstPhysicsObjectHandle Handle)
	{
		if (!Handle)
		{
			return nullptr;
		}

		if (TThreadParticle<Id>* Particle = Handle->GetParticle<Id>())
		{
			return Particle->GetGeometry();
		}

		return nullptr;
	}

	template<EThreadContext Id>
	void FReadPhysicsObjectInterface<Id>::VisitEveryShape(TArrayView<const FConstPhysicsObjectHandle> InObjects, TFunctionRef<bool(const FConstPhysicsObjectHandle, TThreadShapeInstance<Id>*)> Lambda)
	{
		for (const FConstPhysicsObjectHandle Object : InObjects)
		{
			if (!Object)
			{
				continue;
			}

			if (TThreadParticle<Id>* Particle = Object->GetParticle<Id>())
			{
				for (const TUniquePtr<TThreadShapeInstance<Id>>& Shape : Particle->ShapeInstances())
				{
					if (Lambda(Object, Shape.Get()))
					{
						return;
					}
				}
			}
		}
	}

	template<EThreadContext Id>
	bool FReadPhysicsObjectInterface<Id>::AreAllValid(TArrayView<const FConstPhysicsObjectHandle> InObjects)
	{
		bool bCheck = !InObjects.IsEmpty();
		for (const FConstPhysicsObjectHandle Object : InObjects)
		{
			bCheck &= (Object != nullptr && Object->IsValid());
		}
		return bCheck;
	}

	template<EThreadContext Id>
	bool FReadPhysicsObjectInterface<Id>::AreAllKinematic(TArrayView<const FConstPhysicsObjectHandle> InObjects)
	{
		bool bCheck = !InObjects.IsEmpty();
		for (const FConstPhysicsObjectHandle Object : InObjects)
		{
			bCheck &= (Object && Object->IsValid() && Object->ObjectState<Id>() == EObjectStateType::Kinematic);
		}
		return bCheck;
	}

	template<EThreadContext Id>
	bool FReadPhysicsObjectInterface<Id>::AreAllSleeping(TArrayView<const FConstPhysicsObjectHandle> InObjects)
	{
		bool bCheck = !InObjects.IsEmpty();
		for (const FConstPhysicsObjectHandle Object : InObjects)
		{
			bCheck &= (Object && Object->IsValid() && Object->ObjectState<Id>() == EObjectStateType::Sleeping);
		}
		return bCheck;
	}

	template<EThreadContext Id>
	bool FReadPhysicsObjectInterface<Id>::AreAllRigidBody(TArrayView<const FConstPhysicsObjectHandle> InObjects)
	{
		bool bCheck = !InObjects.IsEmpty();
		for (const FConstPhysicsObjectHandle Object : InObjects)
		{
			bCheck &= (Object && Object->IsValid() && Object->ObjectState<Id>() != EObjectStateType::Static);
		}
		return bCheck;
	}

	template<EThreadContext Id>
	bool FReadPhysicsObjectInterface<Id>::AreAllDynamic(TArrayView<const FConstPhysicsObjectHandle> InObjects)
	{
		bool bCheck = !InObjects.IsEmpty();
		for (const FConstPhysicsObjectHandle Object : InObjects)
		{
			bCheck &= (Object && Object->IsValid() && Object->ObjectState<Id>() == EObjectStateType::Dynamic);
		}
		return bCheck;
	}

	template<EThreadContext Id>
	bool FReadPhysicsObjectInterface<Id>::AreAllDynamicOrSleeping(TArrayView<const FConstPhysicsObjectHandle> InObjects)
	{
		bool bCheck = !InObjects.IsEmpty();
		for (const FConstPhysicsObjectHandle Object : InObjects)
		{
			bCheck &= (Object && Object->IsValid() && (Object->ObjectState<Id>() == EObjectStateType::Dynamic || Object->ObjectState<Id>() == EObjectStateType::Sleeping));
		}
		return bCheck;
	}

	template<EThreadContext Id>
	bool FReadPhysicsObjectInterface<Id>::AreAllDisabled(TArrayView<const FConstPhysicsObjectHandle> InObjects)
	{
		bool bDisabled = !InObjects.IsEmpty();
		for (const FConstPhysicsObjectHandle Object : InObjects)
		{
			bool bParticleDisabled = true;
			if (Object)
			{
				if (TThreadParticle<Id>* Particle = Object->GetParticle<Id>())
				{
					bParticleDisabled = FPhysicsObject::IsParticleDisabled<Id>(Particle);
				}
			}
			bDisabled &= bParticleDisabled;
		}
		return bDisabled;
	}

	template<EThreadContext Id>
	bool FReadPhysicsObjectInterface<Id>::AreAllShapesQueryEnabled(TArrayView<const FConstPhysicsObjectHandle> InObjects)
	{
		if (InObjects.IsEmpty())
		{
			return false;
		}

		for (const FConstPhysicsObjectHandle Object : InObjects)
		{
			if (Object)
			{
				if (TThreadParticle<Id>* Particle = Object->GetParticle<Id>())
				{
					for (const TUniquePtr<FPerShapeData>& ShapeData : Particle->ShapesArray())
					{
						if (!ShapeData->GetCollisionData().bQueryCollision)
						{
							return false;
						}
					}
				}
			}
		}
		return true;
	}

	template<EThreadContext Id>
	float FReadPhysicsObjectInterface<Id>::GetMass(TArrayView<const FConstPhysicsObjectHandle> InObjects)
	{
		float Mass = 0.f;
		for (const FConstPhysicsObjectHandle Object : InObjects)
		{
			if (Object)
			{
				if (TThreadParticle<Id>* Particle = Object->GetParticle<Id>())
				{
					if (TThreadRigidParticle<Id>* Rigid = Particle->CastToRigidParticle())
					{
						Mass += static_cast<float>(Rigid->M());
					}
				}
			}
		}
		return Mass;
	}

	template<EThreadContext Id>
	FBox FReadPhysicsObjectInterface<Id>::GetBounds(TArrayView<const FConstPhysicsObjectHandle> InObjects)
	{
		FBox RetBox(ForceInit);
		for (const FConstPhysicsObjectHandle Object : InObjects)
		{
			if (!Object)
			{
				continue;
			}

			const TThreadParticle<Id>* Particle = Object->GetParticle<Id>();
			if (!Particle)
			{
				continue;
			}

			FBox ParticleBox(ForceInit);
			if (const FImplicitObjectRef Geometry = Particle->GetGeometry(); Geometry && Geometry->HasBoundingBox())
			{
				const Chaos::FAABB3 Box = Geometry->BoundingBox();
				ParticleBox = FBox{ Box.Min(), Box.Max() };
			}

			if (ParticleBox.IsValid)
			{
				RetBox += ParticleBox;
			}
		}
		return RetBox;
	}

	template<EThreadContext Id>
	FBox FReadPhysicsObjectInterface<Id>::GetWorldBounds(TArrayView<const FConstPhysicsObjectHandle> InObjects)
	{
		FBox RetBox(ForceInit);
		for (const FConstPhysicsObjectHandle Object : InObjects)
		{
			if (!Object)
			{
				continue;
			}

			const TThreadParticle<Id>* Particle = Object->GetParticle<Id>();
			if (!Particle)
			{
				continue;
			}

			FBox ParticleBox(ForceInit);
			if (const FImplicitObjectRef Geometry = Particle->GetGeometry(); Geometry && Geometry->HasBoundingBox())
			{
				const Chaos::FAABB3 WorldBox = Geometry->CalculateTransformedBounds(TRigidTransform<FReal, 3>(Particle->GetX(), Particle->GetR()));
				ParticleBox = FBox{ WorldBox.Min(), WorldBox.Max() };
			}

			if (ParticleBox.IsValid)
			{
				RetBox += ParticleBox;
			}
		}
		return RetBox;
	}

	template<EThreadContext Id>
	FClosestPhysicsObjectResult FReadPhysicsObjectInterface<Id>::GetClosestPhysicsBodyFromLocation(TArrayView<const FConstPhysicsObjectHandle> InObjects, const FVector& WorldLocation)
	{
		FClosestPhysicsObjectResult AggregateResult;
		for (const FConstPhysicsObjectHandle Object : InObjects)
		{
			if (!Object)
			{
				continue;
			}

			const TThreadParticle<Id>* Particle = Object->GetParticle<Id>();
			if (!Particle)
			{
				continue;
			}

			const FTransform WorldTransform = GetTransform(Object);
			const FVector LocalLocation = WorldTransform.InverseTransformPosition(WorldLocation);

			FClosestPhysicsObjectResult Result;

			if (const FImplicitObjectRef Geometry = Particle->GetGeometry())
			{
				Result.PhysicsObject = const_cast<FPhysicsObjectHandle>(Object);

				Chaos::FVec3 Normal;
				Result.ClosestDistance = static_cast<double>(Geometry->PhiWithNormal(LocalLocation, Normal));
				Result.ClosestLocation = WorldTransform.TransformPosition(LocalLocation - Result.ClosestDistance * Normal);
			}

			if (!Result)
			{
				continue;
			}

			if (!AggregateResult || Result.ClosestDistance < AggregateResult.ClosestDistance)
			{
				AggregateResult = Result;
			}
		}
		return AggregateResult;
	}

	template<EThreadContext Id>
	FAccelerationStructureHandle FReadPhysicsObjectInterface<Id>::CreateAccelerationStructureHandle(const FConstPhysicsObjectHandle InObject)
	{
		return FAccelerationStructureHandle{InObject->GetParticle<Id>()};
	}

	template<EThreadContext Id>
	void FWritePhysicsObjectInterface<Id>::SetUserDefinedEntity(TArrayView<const FPhysicsObjectHandle> InObjects, FChaosUserDefinedEntity* UserDefinedEntity)
	{
		if constexpr (Id == EThreadContext::External)
		{
			for (const FPhysicsObjectHandle Object : InObjects)
			{
				if (!Object)
				{
					continue;
				}

				if (FGeometryParticle* Particle = Object->GetParticle<Id>())
				{
					// Do we already have a user entity appended?
					FChaosUserEntityAppend* UserEntityAppend = FChaosUserData::Get<FChaosUserEntityAppend>(Particle->UserData());
					if (!UserEntityAppend)
					{
						if (UserDefinedEntity)
						{
							UserEntityAppend = new FChaosUserEntityAppend;
							UserEntityAppend->ChaosUserData = reinterpret_cast<FChaosUserData*>(Particle->UserData());
							UserEntityAppend->UserDefinedEntity = UserDefinedEntity;
							Particle->SetUserData(UserEntityAppend);
						}
					}
					else
					{
						if (UserDefinedEntity)
						{
							UserEntityAppend->UserDefinedEntity = UserDefinedEntity; // Overwrite previous used defined entity
						}
						else
						{
							// Restore the particle user data and delete unused memory
							Particle->SetUserData(UserEntityAppend->ChaosUserData);
							delete UserEntityAppend;
						}
					}
				}
			}
		}
		else
		{
			ensureMsgf(false, TEXT("User Defined Entities can only be set by the game thread"));
		}		
	}

	template<EThreadContext Id>
	void FWritePhysicsObjectInterface<Id>::PutToSleep(TArrayView<const FPhysicsObjectHandle> InObjects)
	{
		for (const FPhysicsObjectHandle Object : InObjects)
		{
			if (!Object)
			{
				continue;
			}

			EObjectStateType State = Object->ObjectState<Id>();
			if (State == EObjectStateType::Dynamic || State == EObjectStateType::Sleeping)
			{
				SetParticleStateHelper<Id>(Object, EObjectStateType::Sleeping);
			}
		}
	}

	template<EThreadContext Id>
	void FWritePhysicsObjectInterface<Id>::WakeUp(TArrayView<const FPhysicsObjectHandle> InObjects)
	{
		for (const FPhysicsObjectHandle Object : InObjects)
		{
			if (!Object)
			{
				continue;
			}

			if (TThreadParticle<Id>* Particle = Object->GetParticle<Id>())
			{
				EObjectStateType State = Object->ObjectState<Id>();
				if (State == EObjectStateType::Dynamic || State == EObjectStateType::Sleeping)
				{
					SetParticleStateHelper<Id>(Object, EObjectStateType::Dynamic);
					if constexpr (Id == EThreadContext::External)
					{
						if (TThreadRigidParticle<Id>* Rigid = Particle->CastToRigidParticle())
						{
							Rigid->ClearEvents();
						}
					}
				}
			}
		}
	}

	template<EThreadContext Id>
	void FWritePhysicsObjectInterface<Id>::ForceKinematic(TArrayView<const FPhysicsObjectHandle> InObjects)
	{
		for (const FPhysicsObjectHandle Object : InObjects)
		{
			if (!Object)
			{
				continue;
			}

			if (TThreadParticle<Id>* Particle = Object->GetParticle<Id>())
			{
				EObjectStateType State = Object->ObjectState<Id>();
				if (State != EObjectStateType::Kinematic)
				{
					SetParticleStateHelper<Id>(Object, EObjectStateType::Kinematic);
				}
			}
		}
	}

	template<EThreadContext Id>
	void FWritePhysicsObjectInterface<Id>::AddForce(TArrayView<const FPhysicsObjectHandle> InObjects, const FVector& Force, bool bInvalidate)
	{
		for (const FPhysicsObjectHandle Object : InObjects)
		{
			if (!Object)
			{
				continue;
			}

			if (TThreadParticle<Id>* Particle = Object->GetParticle<Id>())
			{
				if (Chaos::TThreadRigidParticle<Id>* Rigid = Particle->CastToRigidParticle())
				{
					if (Rigid->ObjectState() == EObjectStateType::Sleeping || Rigid->ObjectState() == EObjectStateType::Dynamic)
					{
						if (bInvalidate)
						{
							SetParticleStateHelper<Id>(Object, EObjectStateType::Dynamic);
						}

						Rigid->AddForce(Force, bInvalidate);
					}
				}
			}
		}

		if constexpr (Id == EThreadContext::External)
		{
			if (Chaos::FPhysicsSolverBase* Solver = GetSolverForObjects(InObjects))
			{
				Solver->EnqueueCommandImmediate(
					[AllObjects = TArray<FPhysicsObjectHandle>{ InObjects }, Force, bInvalidate]() {
						Chaos::FWritePhysicsObjectInterface_Internal Interface = Chaos::FPhysicsObjectInternalInterface::GetWrite();
						Interface.AddForce(AllObjects, Force, bInvalidate);
					}
				);
			}
		}
	}

	template<EThreadContext Id>
	void FWritePhysicsObjectInterface<Id>::AddTorque(TArrayView<const FPhysicsObjectHandle> InObjects, const FVector& Torque, bool bInvalidate)
	{
		AddTorque(InObjects, Torque, bInvalidate, false);
	}

	template<EThreadContext Id>
	void FWritePhysicsObjectInterface<Id>::AddTorque(TArrayView<const FPhysicsObjectHandle> InObjects, const FVector& Torque, bool bInvalidate, bool bAccelChange)
	{
		for (const FPhysicsObjectHandle Object : InObjects)
		{
			if (!Object)
			{
				continue;
			}

			if (TThreadParticle<Id>* Particle = Object->GetParticle<Id>())
			{
				if (Chaos::TThreadRigidParticle<Id>* Rigid = Particle->CastToRigidParticle())
				{
					if (Rigid->ObjectState() == EObjectStateType::Sleeping || Rigid->ObjectState() == EObjectStateType::Dynamic)
					{
						if (bInvalidate)
						{
							SetParticleStateHelper<Id>(Object, EObjectStateType::Dynamic);
						}

						if (bAccelChange)
						{
							FRotation3 QCom = Rigid->GetR() * Rigid->RotationOfMass();
							if constexpr (Id == EThreadContext::Internal)
							{
								QCom = Rigid->GetQ() * Rigid->RotationOfMass();
							}
							const FMatrix33 WorldInertia = Utilities::ComputeWorldSpaceInertia(QCom, Rigid->I());
							Rigid->AddTorque(WorldInertia * Torque, bInvalidate);
						}
						else 
						{
							Rigid->AddTorque(Torque, bInvalidate);
						}
					}
				}
			}
		}

		if constexpr (Id == EThreadContext::External)
		{
			if (Chaos::FPhysicsSolverBase* Solver = GetSolverForObjects(InObjects))
			{
				Solver->EnqueueCommandImmediate(
					[AllObjects = TArray<FPhysicsObjectHandle>{ InObjects }, Torque, bInvalidate, bAccelChange]() {
						Chaos::FWritePhysicsObjectInterface_Internal Interface = Chaos::FPhysicsObjectInternalInterface::GetWrite();
						Interface.AddTorque(AllObjects, Torque, bInvalidate, bAccelChange);
					}
				);
			}
		}
	}

	template<EThreadContext Id>
	void FWritePhysicsObjectInterface<Id>::SetLinearVelocity(TArrayView<const FPhysicsObjectHandle> InObjects, const FVector& InVelocity, bool bAddToCurrent)
	{
		if (InObjects.IsEmpty())
		{
			return;
		}

		for (const FPhysicsObjectHandle Object : InObjects)
		{
			if (!Object)
			{
				continue;
			}

			if (TThreadParticle<Id>* Particle = Object->GetParticle<Id>())
			{
				if (Chaos::TThreadRigidParticle<Id>* Rigid = Particle->CastToRigidParticle())
				{
					FVector Final = InVelocity;

					if (bAddToCurrent)
					{
						Final += Rigid->GetV();
					}

					Rigid->SetV(Final, false);
				}
			}
		}

		if constexpr (Id == EThreadContext::External)
		{
			if (Chaos::FPhysicsSolverBase* Solver = GetSolverForObjects(InObjects))
			{
				Solver->EnqueueCommandImmediate(
					[AllObjects = TArray<FPhysicsObjectHandle>{ InObjects }, InVelocity, bAddToCurrent]() {
						Chaos::FWritePhysicsObjectInterface_Internal Interface = Chaos::FPhysicsObjectInternalInterface::GetWrite();
						Interface.SetLinearVelocity(AllObjects, InVelocity, bAddToCurrent);
					}
				);
			}
		}
	}

	template<EThreadContext Id>
	void FWritePhysicsObjectInterface<Id>::SetAngularVelocityInRadians(TArrayView<const FPhysicsObjectHandle> InObjects, const FVector& InAngularVelocity, bool bAddToCurrent)
	{
		if (InObjects.IsEmpty())
		{
			return;
		}

		for (const FPhysicsObjectHandle Object : InObjects)
		{
			if (!Object)
			{
				continue;
			}

			if (TThreadParticle<Id>* Particle = Object->GetParticle<Id>())
			{
				if (Chaos::TThreadRigidParticle<Id>* Rigid = Particle->CastToRigidParticle())
				{
					FVector Final = InAngularVelocity;

					if (bAddToCurrent)
					{
						Final += Rigid->GetW();
					}

					Rigid->SetW(Final, false);
				}
			}
		}

		if constexpr (Id == EThreadContext::External)
		{
			if (Chaos::FPhysicsSolverBase* Solver = GetSolverForObjects(InObjects))
			{
				Solver->EnqueueCommandImmediate(
					[AllObjects = TArray<FPhysicsObjectHandle>{ InObjects }, InAngularVelocity, bAddToCurrent]() {
						Chaos::FWritePhysicsObjectInterface_Internal Interface = Chaos::FPhysicsObjectInternalInterface::GetWrite();
						Interface.SetAngularVelocityInRadians(AllObjects, InAngularVelocity, bAddToCurrent);
					}
				);
			}
		}
	}

	template<EThreadContext Id>
	void FWritePhysicsObjectInterface<Id>::SetLinearImpulseVelocity(TArrayView<const FPhysicsObjectHandle> InObjects, const FVector& Impulse, bool bVelChange)
	{
		if (InObjects.IsEmpty())
		{
			return;
		}

		for (const FPhysicsObjectHandle Object : InObjects)
		{
			if (!Object)
			{
				continue;
			}

			if (TThreadParticle<Id>* Particle = Object->GetParticle<Id>())
			{
				if (Chaos::TThreadRigidParticle<Id>* Rigid = Particle->CastToRigidParticle())
				{
					if (bVelChange)
					{
						Rigid->SetLinearImpulseVelocity(Impulse, false);
					}
					else
					{
						Rigid->SetLinearImpulseVelocity(Impulse * Rigid->InvM(), false);
					}
				}
			}
		}

		if constexpr (Id == EThreadContext::External)
		{
			if (Chaos::FPhysicsSolverBase* Solver = GetSolverForObjects(InObjects))
			{
				Solver->EnqueueCommandImmediate(
					[AllObjects = TArray<FPhysicsObjectHandle>{InObjects}, Impulse, bVelChange]() {
						Chaos::FWritePhysicsObjectInterface_Internal Interface = Chaos::FPhysicsObjectInternalInterface::GetWrite();
						Interface.SetLinearImpulseVelocity(AllObjects, Impulse, bVelChange);
					}
				);
			}
		}
	}

	template<EThreadContext Id>
	void FWritePhysicsObjectInterface<Id>::AddRadialImpulse(TArrayView<const FPhysicsObjectHandle> InObjects, FVector Origin, float Radius, float Strength, enum ERadialImpulseFalloff Falloff, bool bApplyStrain, bool bInvalidate, bool bVelChange)
	{
		// passing -1.0f as a strain will make use of the strain modifier instead ( legacy system )
		AddRadialImpulse(InObjects, Origin, Radius, Strength, Falloff, bApplyStrain, -1.0f, bInvalidate, bVelChange);
	}

	template<EThreadContext Id>
	void FWritePhysicsObjectInterface<Id>::AddRadialImpulse(TArrayView<const FPhysicsObjectHandle> InObjects, FVector Origin, float Radius, float Strength, enum ERadialImpulseFalloff Falloff, bool bApplyStrain, float Strain, bool bInvalidate, bool bVelChange, float MinValue, float MaxValue)
	{
		//TODO: create a PT version of this, plus the damping functions
		if (Chaos::FPBDRigidsSolver* RigidSolver = Chaos::FPhysicsObjectInterface::GetSolver(InObjects))
		{
			//put onto physics thread
			RigidSolver->EnqueueCommandImmediate([InObjects = TArray<FPhysicsObjectHandle>{ InObjects },
				RigidSolver,
				Origin,
				Radius,
				Strength,
				Falloff,
				bApplyStrain,
				Strain,
				bInvalidate,
				bVelChange,
				MinValue,
				MaxValue
				]()
				{
					using namespace Chaos;
					for (FPhysicsObjectHandle Object : InObjects)
					{
						if (Object == nullptr)
						{
							continue;
						}

						// can apply to clusters or rigid particles
						if (TThreadParticle<EThreadContext::Internal>* Particle = Object->GetParticle<EThreadContext::Internal>())
						{
							FPBDRigidClusteredParticleHandle* Cluster = Particle->CastToClustered();
							if (Cluster && bApplyStrain)
							{
								// it is a cluster and we are applying strain so we need to get to the children particles
								FRigidClustering& Clustering = RigidSolver->GetEvolution()->GetRigidClustering();
								TArray<FPBDRigidParticleHandle*>* ChildrenHandles = Clustering.GetChildrenMap().Find(Cluster);

								if (ChildrenHandles == nullptr)
								{
									continue;
								}

								float VelocityRatio = 1.0f;
								if (PhysicsObjectInterfaceCVars::RadialImpulseDistributeToChildren)
								{
									VelocityRatio = 1.0f / static_cast<float>(ChildrenHandles->Num());
								}

								for (FPBDRigidParticleHandle* ChildHandle : *ChildrenHandles)
								{
									const float FalloffAlpha = AddRadialImpulseHelper(ChildHandle, Origin, Radius, Strength, Falloff, VelocityRatio, bInvalidate, bVelChange, MinValue, MaxValue);

									// todo(chaos) : Remove StrainModifier cvar when material system is in place and densities are updated
									const float StrainToApply =
										(Strain < 0)
										? (PhysicsObjectInterfaceCVars::StrainModifier * FalloffAlpha * Strength)
										: Strain * FalloffAlpha;
									if (StrainToApply > 0)
									{
										Clustering.SetExternalStrain(ChildHandle->CastToClustered(), StrainToApply);
									}
								}
							}
							else
							{
								// apply the impulse to the particle itself 
								FPBDRigidParticleHandle* ParticleHandle = Particle->CastToRigidParticle();
								if (ParticleHandle != nullptr && ParticleHandle->IsSleeping())
								{
									RigidSolver->GetEvolution()->SetParticleObjectState(ParticleHandle, Chaos::EObjectStateType::Dynamic);
									RigidSolver->GetEvolution()->GetParticles().MarkTransientDirtyParticle(ParticleHandle);
								}
								constexpr float VelocityRatio = 1.f;
								AddRadialImpulseHelper(ParticleHandle, Origin, Radius, Strength, Falloff, VelocityRatio, bInvalidate, bVelChange, MinValue, MaxValue);
							}
						}
					}
				});
		}
	}

	template<EThreadContext Id>
	void Chaos::FWritePhysicsObjectInterface<Id>::SetLinearEtherDrag(TArrayView<const FPhysicsObjectHandle> InObjects, float InLinearDrag)
	{
		if (InObjects.IsEmpty())
		{
			return;
		}

		for (const FPhysicsObjectHandle Object : InObjects)
		{
			if (!Object)
			{
				continue;
			}

			if (TThreadParticle<Id>* Particle = Object->GetParticle<Id>())
			{
				if (Chaos::TThreadRigidParticle<Id>* Rigid = Particle->CastToRigidParticle())
				{
					Rigid->SetLinearEtherDrag(InLinearDrag);
				}
			}
		}
			
		if constexpr (Id == EThreadContext::External)
		{
			if (Chaos::FPBDRigidsSolver* RigidSolver = Chaos::FPhysicsObjectInterface::GetSolver(InObjects))
			{
				//put onto physics thread
				RigidSolver->EnqueueCommandImmediate(
					[InObjects = TArray<FPhysicsObjectHandle>{ InObjects }, InLinearDrag]() {
						Chaos::FWritePhysicsObjectInterface_Internal Interface = Chaos::FPhysicsObjectInternalInterface::GetWrite();
						Interface.SetLinearEtherDrag(InObjects, InLinearDrag);
					});
			}
		}
	}


	template<EThreadContext Id>
	void Chaos::FWritePhysicsObjectInterface<Id>::SetAngularEtherDrag(TArrayView<const FPhysicsObjectHandle> InObjects, float InAngularDrag)
	{
		if (InObjects.IsEmpty())
		{
			return;
		}

		for (const FPhysicsObjectHandle Object : InObjects)
		{
			if (!Object)
			{
				continue;
			}

			if (TThreadParticle<Id>* Particle = Object->GetParticle<Id>())
			{
				if (Chaos::TThreadRigidParticle<Id>* Rigid = Particle->CastToRigidParticle())
				{
					Rigid->SetAngularEtherDrag(InAngularDrag);
				}
			}
		}

		if constexpr (Id == EThreadContext::External)
		{
			if (Chaos::FPBDRigidsSolver* RigidSolver = Chaos::FPhysicsObjectInterface::GetSolver(InObjects))
			{
				//put onto physics thread
				RigidSolver->EnqueueCommandImmediate(
					[InObjects = TArray<FPhysicsObjectHandle>{ InObjects }, InAngularDrag]() {
						Chaos::FWritePhysicsObjectInterface_Internal Interface = Chaos::FPhysicsObjectInternalInterface::GetWrite();
						Interface.SetAngularEtherDrag(InObjects, InAngularDrag);
					});
			}
		}
	}

	template<EThreadContext Id>
	void FWritePhysicsObjectInterface<Id>::UpdateShapeCollisionFlags(TArrayView<const FPhysicsObjectHandle> InObjects, bool bSimCollision, bool bQueryCollision)
	{
		for (const FPhysicsObjectHandle Object : InObjects)
		{
			if (!Object)
			{
				continue;
			}

			if (TThreadParticle<Id>* Particle = Object->GetParticle<Id>())
			{
				for (const TUniquePtr<Chaos::FPerShapeData>& ShapeData : Particle->ShapesArray())
				{
					FCollisionData Data = ShapeData->GetCollisionData();
					Data.bSimCollision = bSimCollision;
					Data.bQueryCollision = bQueryCollision;
					ShapeData->SetCollisionData(Data);
				}
			}
		}
	}

	template<EThreadContext Id>
	void FWritePhysicsObjectInterface<Id>::UpdateShapeFilterData(TArrayView<const FPhysicsObjectHandle> InObjects, const FCollisionFilterData& QueryData, const FCollisionFilterData& SimData)
	{
		Filter::FCombinedShapeFilterData ShapeFilterData = Chaos::Filter::FShapeFilterBuilder::BuildFromLegacyShapeFilter(QueryData, SimData);
		UpdateCombinedShapeFilterData(InObjects, ShapeFilterData);
	}

	template<EThreadContext Id>
	void FWritePhysicsObjectInterface<Id>::UpdateCombinedShapeFilterData(TArrayView<const FPhysicsObjectHandle> InObjects, const Chaos::Filter::FCombinedShapeFilterData& CombinedShapeFilter)
	{
		for (const FPhysicsObjectHandle Object : InObjects)
		{
			if (!Object)
			{
				continue;
			}

			if (TThreadParticle<Id>* Particle = Object->GetParticle<Id>())
			{
				for (const TUniquePtr<Chaos::FPerShapeData>& ShapeData : Particle->ShapesArray())
				{
					ShapeData->SetCombinedShapeFilterData(CombinedShapeFilter);
				}
			}
		}
	}

	template<EThreadContext Id>
	void FWritePhysicsObjectInterface<Id>::AddConnectivityEdgesBetween(TArrayView<const FPhysicsObjectHandle> FromObjects, TArrayView<const FPhysicsObjectHandle> ToObjects, const FClusterCreationParameters& Parameters)
	{
		if (FromObjects.IsEmpty() || ToObjects.IsEmpty())
		{
			return;
		}

		// Assume that the solver for both sets of objects are the same.
		if (FPBDRigidsSolver* Solver = FPhysicsObjectInterface::GetSolver(FromObjects))
		{
			if constexpr (Id == EThreadContext::External)
			{
				Solver->EnqueueCommandImmediate(
					[FromObjects = TArray<FPhysicsObjectHandle>{ FromObjects }, ToObjects = TArray<FPhysicsObjectHandle>{ ToObjects }, Parameters]()
					{
						FWritePhysicsObjectInterface_Internal Internal = FPhysicsObjectInternalInterface::GetWrite();
						Internal.AddConnectivityEdgesBetween(FromObjects, ToObjects, Parameters);
					}
				);
			}
			else
			{
				FRigidClustering& Clustering = Solver->GetEvolution()->GetRigidClustering();

				TArray<FPBDRigidParticleHandle*> AllParticles;
				AllParticles.Reserve(FromObjects.Num() + ToObjects.Num());

				TArray<FPBDRigidParticleHandle*> FromParticles = this->GetAllRigidParticles(FromObjects);
				TArray<FPBDRigidParticleHandle*> ToParticles = this->GetAllRigidParticles(ToObjects);
				AllParticles.Append(FromParticles);
				AllParticles.Append(ToParticles);

				TSet<FPBDRigidParticleHandle*> FromSet{ FromParticles };
				TSet<FPBDRigidParticleHandle*> ToSet{ ToParticles };
				Clustering.GenerateConnectionGraph(AllParticles, Parameters, &FromSet, &ToSet);
			}
		}
		
	}

	void FPhysicsObjectInterface::SetName(const FPhysicsObjectHandle Object, const FName& InName)
	{
		if (!Object)
		{
			return;
		}

		Object->SetName(InName);
	}

	FName FPhysicsObjectInterface::GetName(const FConstPhysicsObjectHandle Object)
	{
		if (!Object)
		{
			return NAME_None;
		}

		return Object->GetBodyName();
	}

#if CHAOS_DEBUG_NAME
	const FString& FPhysicsObjectInterface::GetDebugName(const FConstPhysicsObjectHandle Object)
	{
		TThreadParticle<EThreadContext::External>* Particle = Object ? Object->GetParticle<EThreadContext::External>() : nullptr;
		if (!Object)
		{
			return Private::EmptyParticleName;
		}

		return Particle->GetDebugName();
	}
#endif

	void FPhysicsObjectInterface::SetId(const FPhysicsObjectHandle Object, int32 InId)
	{
		if (!Object)
		{
			return;
		}

		Object->SetBodyIndex(InId);
	}

	int32 FPhysicsObjectInterface::GetId(const FConstPhysicsObjectHandle Object)
	{
		if (!Object)
		{
			return INDEX_NONE;
		}

		return Object->GetBodyIndex();
	}

	FPBDRigidsSolver* FPhysicsObjectInterface::GetSolver(TArrayView<const FConstPhysicsObjectHandle> InObjects)
	{
		FPBDRigidsSolver* RetSolver = nullptr;
		for (const FConstPhysicsObjectHandle Object : InObjects)
		{
			if (!Object)
			{
				continue;
			}

			FPBDRigidsSolver* Solver = GetSolver(Object);

			if (!Solver)
			{
				return nullptr;
			}
			else if (!RetSolver)
			{
				RetSolver = Solver;
			}
			else if (Solver != RetSolver)
			{
				return nullptr;
			}
		}
		return RetSolver;
	}

	FPBDRigidsSolver* FPhysicsObjectInterface::GetSolver(const FConstPhysicsObjectHandle InObject)
	{
		if (!InObject)
		{
			return nullptr;
		}

		FPBDRigidsSolver* Solver = nullptr;
		if (const IPhysicsProxyBase* Proxy = InObject->PhysicsProxy())
		{
			Solver = Proxy->GetSolver<FPBDRigidsSolver>();
		}

		return Solver;
	}


	IPhysicsProxyBase* FPhysicsObjectInterface::GetProxy(TArrayView<const FConstPhysicsObjectHandle> InObjects)
	{
		IPhysicsProxyBase* RetProxy = nullptr;
		for (const FConstPhysicsObjectHandle Object : InObjects)
		{
			if (!Object)
			{
				continue;
			}

			IPhysicsProxyBase* Proxy = const_cast<IPhysicsProxyBase*>(Object->PhysicsProxy());
			if (!Proxy)
			{
				return nullptr;
			}
			else if (!RetProxy)
			{
				RetProxy = Proxy;
			}
			else if (Proxy != RetProxy)
			{
				return nullptr;
			}
		}
		return RetProxy;
	}

	template class FReadPhysicsObjectInterface<EThreadContext::External>;
	template class FReadPhysicsObjectInterface<EThreadContext::Internal>;

	template class FWritePhysicsObjectInterface<EThreadContext::External>;
	template class FWritePhysicsObjectInterface<EThreadContext::Internal>;
}
