// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Chaos/PhysicsObject.h"
#include "Chaos/ParticleHandleFwd.h"
#include "Chaos/ShapeInstanceFwd.h"
#include "Containers/Set.h"
#include "Framework/Threading.h"

#include "PhysicsProxy/ClusterUnionPhysicsProxy.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "PhysicsProxy/GeometryCollectionPhysicsProxy.h"

namespace Chaos
{
	struct FPhysicsObject
	{
	public:
		bool IsValid() const;

		void SetBodyIndex(int32 InBodyIndex) { BodyIndex = InBodyIndex; }
		int32 GetBodyIndex() const { return BodyIndex; }

		void SetName(const FName& InBodyName) { BodyName = InBodyName; }
		const FName& GetBodyName() const { return BodyName; }

		template<EThreadContext Id>
		EObjectStateType ObjectState() const
		{
			TThreadParticle<Id>* Particle = GetParticle<Id>();
			if (!Particle)
			{
				return EObjectStateType::Uninitialized;
			}

			if (TThreadRigidParticle<Id>* Rigid = Particle->CastToRigidParticle())
			{
				return Rigid->ObjectState();
			}
			else
			{
				return Particle->ObjectState();
			}
		}

		template<EThreadContext Id>
		FPhysicsObjectHandle GetRootObject() const
		{
			EPhysicsProxyType ProxyType = Proxy->GetType();

			TThreadParticle<Id>* Particle = GetParticle<Id>();
			FPhysicsObjectHandle CurrentObject = const_cast<FPhysicsObjectHandle>(this);
			FPhysicsObjectHandle CurrentParent = GetParentObject<Id>();
			while (CurrentParent && IsParticleDisabled<Id>(Particle))
			{
				Particle = CurrentParent->GetParticle<Id>();
				CurrentObject = CurrentParent;
				CurrentParent = CurrentParent->GetParentObject<Id>();
			}
			return CurrentObject;
		}

		template<EThreadContext Id>
		FPhysicsObjectHandle GetParentObject() const
		{
			EPhysicsProxyType ProxyType = Proxy->GetType();
			switch (ProxyType)
			{
			case EPhysicsProxyType::GeometryCollectionType:
			{
				FGeometryCollectionPhysicsProxy* GeometryCollectionProxy = static_cast<FGeometryCollectionPhysicsProxy*>(Proxy);
				FGeometryDynamicCollection& Collection = GetGeometryCollectionDynamicCollection<Id>(*GeometryCollectionProxy);

				if (int32 Index = Collection.GetParent(BodyIndex); Index != INDEX_NONE)
				{
					return GeometryCollectionProxy->GetPhysicsObjectByIndex(Index);
				}

				// Yes this is an explicit fall through. Once we get to the root particle on the geometry collection we need to check
				// if it's in a cluster union same with all the other proxies.
				[[fallthrough]];
			}
			case EPhysicsProxyType::ClusterUnionProxy:
			case EPhysicsProxyType::SingleParticleProxy:
			{
				// At this point, we need to be able to check if we're part of a cluster union.
				// However, note that we're after more than just the parent particle, we're after
				// the parent physics object handle which lives on the proxy. Hence why physics proxies
				// need to keep track of their parent proxy. We do, however, make the assumption that the
				// only possible valid parent proxy is a cluster union proxy though; hence being able to directly
				// retrieve the physics object handle.
				if (IPhysicsProxyBase* ParentProxy = Proxy->GetParentProxy(); ParentProxy && ParentProxy->GetType() == EPhysicsProxyType::ClusterUnionProxy)
				{
					FClusterUnionPhysicsProxy* ParentClusterProxy = static_cast<FClusterUnionPhysicsProxy*>(ParentProxy);
					return ParentClusterProxy->GetPhysicsObjectHandle();
				}
				break;
			}
			default:
				break;
			}
			return nullptr;
		}

		template <EThreadContext Id>
		TThreadParticle<Id>* GetRootParticle() const
		{
			FPhysicsObjectHandle Root = GetRootObject<Id>();
			if (!Root)
			{
				return nullptr;
			}
			return Root->GetParticle<Id>();
		}

		template <EThreadContext Id>
		TThreadParticle<Id>* GetParticle() const
		{
			if (Proxy)
			{
				EPhysicsProxyType ProxyType = Proxy->GetType();
				if constexpr (Id == EThreadContext::External)
				{
					switch (ProxyType)
					{
					case EPhysicsProxyType::ClusterUnionProxy:
						return static_cast<FClusterUnionPhysicsProxy*>(Proxy)->GetParticle_External();
					case EPhysicsProxyType::GeometryCollectionType:
						return static_cast<FGeometryCollectionPhysicsProxy*>(Proxy)->GetParticleByIndex_External(BodyIndex);
					case EPhysicsProxyType::SingleParticleProxy:
						return static_cast<FSingleParticlePhysicsProxy*>(Proxy)->GetParticle_LowLevel();
					default:
						break;
					}
				}
				else
				{
					switch (ProxyType)
					{
					case EPhysicsProxyType::ClusterUnionProxy:
						return static_cast<FClusterUnionPhysicsProxy*>(Proxy)->GetParticle_Internal();
					case EPhysicsProxyType::GeometryCollectionType:
						return static_cast<FGeometryCollectionPhysicsProxy*>(Proxy)->GetParticleByIndex_Internal(BodyIndex);
					case EPhysicsProxyType::SingleParticleProxy:
						return static_cast<FSingleParticlePhysicsProxy*>(Proxy)->GetHandle_LowLevel();
					default:
						break;
					}
				}
			}

			return (TThreadParticle<Id>*)nullptr;
		}

		template <EThreadContext Id>
		static bool IsParticleDisabled(TThreadParticle<Id>* Particle)
		{
			if (TThreadRigidParticle<Id>* Rigid = Particle->CastToRigidParticle())
			{
				return Rigid->Disabled();
			}
			return false;
		}

		IPhysicsProxyBase* PhysicsProxy() { return Proxy; }
		const IPhysicsProxyBase* PhysicsProxy() const { return Proxy; }

		template<EThreadContext Id>
		bool HasChildren() const
		{
			EPhysicsProxyType ProxyType = Proxy->GetType();
			switch (ProxyType)
			{
			case EPhysicsProxyType::GeometryCollectionType:
			{
				FGeometryDynamicCollection& Collection = GetGeometryCollectionDynamicCollection<Id>(*static_cast<FGeometryCollectionPhysicsProxy*>(Proxy));
				return Collection.HasChildren(BodyIndex);
			}
			case EPhysicsProxyType::ClusterUnionProxy:
			{
				FClusterUnionPhysicsProxy* ClusterProxy = static_cast<FClusterUnionPhysicsProxy*>(Proxy);
				if constexpr (Id == EThreadContext::External)
				{
					return ClusterProxy->HasChildren_External();
				}
				else
				{
					return ClusterProxy->HasChildren_Internal();
				}
			}
			default:
				break;
			}
			return false;
		}

		friend class FPhysicsObjectFactory;
	protected:
		FPhysicsObject(IPhysicsProxyBase* InProxy, int32 InBodyIndex = INDEX_NONE, const FName& InBodyName = NAME_None)
			: Proxy(InProxy)
			, BodyIndex(InBodyIndex)
			, BodyName(InBodyName)
		{}

	private:
		IPhysicsProxyBase* Proxy = nullptr;
		int32 BodyIndex = INDEX_NONE;
		FName BodyName = NAME_None;

		template<EThreadContext Id>
		static FGeometryDynamicCollection& GetGeometryCollectionDynamicCollection(FGeometryCollectionPhysicsProxy& GeometryCollectionProxy)
		{
			if constexpr (Id == EThreadContext::External)
			{
				return GeometryCollectionProxy.GetExternalCollection();
			}
			else
			{
				return GeometryCollectionProxy.GetPhysicsCollection();
			}
		}
	};

	/**
	 * This class helps us limit the creation of the FPhysicsObject to only internal use cases. Realistically, only the physics proxies
	 * should be creating physics objects.
	 */
	class FPhysicsObjectFactory
	{
	public:
		static FPhysicsObjectUniquePtr CreatePhysicsObject(IPhysicsProxyBase* InProxy, int32 InBodyIndex = INDEX_NONE, const FName& InBodyName = NAME_None);
	};
}