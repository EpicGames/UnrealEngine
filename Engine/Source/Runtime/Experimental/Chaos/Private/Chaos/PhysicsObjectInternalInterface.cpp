// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/PhysicsObjectInternalInterface.h"

namespace Chaos
{
	FReadPhysicsObjectInterface_Internal FPhysicsObjectInternalInterface::GetRead()
	{
		return CreateReadInterface<EThreadContext::Internal>();
	}

	FWritePhysicsObjectInterface_Internal FPhysicsObjectInternalInterface::GetWrite()
	{
		return CreateWriteInterface<EThreadContext::Internal>();
	}
}