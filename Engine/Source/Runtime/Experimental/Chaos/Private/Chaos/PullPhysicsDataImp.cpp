// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/PullPhysicsDataImp.h"

namespace Chaos
{

void FPullPhysicsData::Reset()
{
	DirtyRigids.Reset();
	DirtyRigidErrors.Reset();
	DirtyGeometryCollections.Reset();
	DirtyClusterUnions.Reset();
	DirtyJointConstraints.Reset();
	DirtyCharacterGroundConstraints.Reset();
}

}