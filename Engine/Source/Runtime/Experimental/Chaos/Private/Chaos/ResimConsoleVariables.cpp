// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/ResimConsoleVariables.h"
#include "HAL/IConsoleManager.h"

namespace Chaos
{
	namespace ResimConsoleVars
	{
		bool bIsBubbleResimEnabled = false;
		static FAutoConsoleVariableRef CVarIsBubbleResimEnabled(
			TEXT("p.Chaos.BubbleResim.Enable"),
			bIsBubbleResimEnabled,
			TEXT("If true bubble resim is enabled, otherwise a resim causes the whole world to resimulate, not just bubbles around server corrected particles"));

		bool bCanFrozenParticlesModifyContacts = true;
		static FAutoConsoleVariableRef CVarCanFrozenParticlesModifyContacts(
			TEXT("p.Chaos.BubbleResim.EnableCanFrozenParticlesModifyContacts"),
			bCanFrozenParticlesModifyContacts,
			TEXT("Whether frozen particles are still allowed to modify contacts during a resim."));
	}
}