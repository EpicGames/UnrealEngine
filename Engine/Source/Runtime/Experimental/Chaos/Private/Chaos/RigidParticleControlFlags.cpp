// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/RigidParticleControlFlags.h"

#include "Chaos/ChaosArchive.h"

#include "Containers/UnrealString.h"

namespace Chaos
{
	FString FRigidParticleControlFlags::ToString() const
	{
		return FString::Printf(TEXT("0x%x"), Bits);
	}

	FChaosArchive& operator<<(FChaosArchive& Ar, FRigidParticleControlFlags& Flags)
	{
		Ar << Flags.Bits;
		return Ar;
	}
}