// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/RigidParticles.h"

//Note this has to be in the cpp to avoid allocating/freeing across DLLs

namespace Chaos
{
	template<typename T, int d>
	void TRigidParticles<T, d>::CollisionParticlesInitIfNeeded(const int32 Index)
	{
		if(MCollisionParticles[Index] == nullptr)
		{
			MCollisionParticles[Index] = MakeUnique<TBVHParticles<T, d>>();
		}
	}

	template<typename T, int d>
	void TRigidParticles<T, d>::SetCollisionParticles(const int32 Index, TParticles<T, d>&& Points)
	{
		MCollisionParticles[Index] = MakeUnique<TBVHParticles<T, d>>(MoveTemp(Points));
	}
}

template class Chaos::TRigidParticles<Chaos::FReal, 3>;
