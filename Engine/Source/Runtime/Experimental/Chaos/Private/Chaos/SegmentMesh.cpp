// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/SegmentMesh.h"
#include "Chaos/Particles.h"

namespace Chaos
{
	FSegmentMesh::FSegmentMesh(TArray<TVec2<int32>>&& Elements)
		: MElements(MoveTemp(Elements))
	{
		// Check for degenerate edges.
		for(const TVec2<int32>& edge : MElements)
		{
			check(edge[0] != edge[1]);
		}
	}

	FSegmentMesh::~FSegmentMesh()
	{}

	void FSegmentMesh::_ClearAuxStructures()
	{
		MPointToEdgeMap.Empty();
		MPointToNeighborsMap.Empty();
	}

	void FSegmentMesh::Init(const TArray<TVec2<int32>>& Elements)
	{
		_ClearAuxStructures();
		MElements = Elements;
		// Check for degenerate edges.
		for(const TVec2<int32>& edge : MElements)
		{
			check(edge[0] != edge[1]);
		}
	}

	void FSegmentMesh::Init(TArray<TVec2<int32>>&& Elements)
	{
		_ClearAuxStructures();
		MElements = MoveTempIfPossible(Elements);
		// Check for degenerate edges.
		for(const TVec2<int32>& edge : MElements)
		{
			check(edge[0] != edge[1]);
		}
	}

	const TMap<int32, TSet<int32>>& FSegmentMesh::GetPointToNeighborsMap() const
	{
		if(!MPointToNeighborsMap.Num())
			_UpdatePointToNeighborsMap();
		return MPointToNeighborsMap;
	}

	void FSegmentMesh::_UpdatePointToNeighborsMap() const
	{
		MPointToNeighborsMap.Reset();
		MPointToNeighborsMap.Reserve(MElements.Num() * 2);
		for(const TVec2<int32>& edge : MElements)
		{
			MPointToNeighborsMap.FindOrAdd(edge[0]).Add(edge[1]);
			MPointToNeighborsMap.FindOrAdd(edge[1]).Add(edge[0]);
			// Paranoia:
			check(MPointToNeighborsMap.Find(edge[0]) != nullptr);
			check(MPointToNeighborsMap.Find(edge[1]) != nullptr);
			check(MPointToNeighborsMap.Find(edge[0])->Find(edge[1]) != nullptr);
			check(MPointToNeighborsMap.Find(edge[1])->Find(edge[0]) != nullptr);
		}
	}

	const TMap<int32, TArray<int32>>& FSegmentMesh::GetPointToEdges() const
	{
		if(!MPointToEdgeMap.Num())
			_UpdatePointToEdgesMap();
		return MPointToEdgeMap;
	}

	void FSegmentMesh::_UpdatePointToEdgesMap() const
	{
		MPointToEdgeMap.Reset();
		MPointToEdgeMap.Reserve(MElements.Num() * 2);
		for(int32 i = 0; i < MElements.Num(); i++)
		{
			const TVec2<int32>& edge = MElements[i];
			MPointToEdgeMap.FindOrAdd(edge[0]).Add(i);
			MPointToEdgeMap.FindOrAdd(edge[1]).Add(i);
		}
	}

	TArray<FReal> FSegmentMesh::GetEdgeLengths(const TParticles<FReal, 3>& InParticles, const bool lengthSquared) const
	{
		TArray<FReal> lengths;
		lengths.AddUninitialized(MElements.Num());
		if(lengthSquared)
		{
			for(int32 i = 0; i < MElements.Num(); i++)
			{
				const TVec2<int32>& edge = MElements[i];
				lengths[i] = (InParticles.GetX(edge[0]) - InParticles.GetX(edge[1])).SizeSquared();
			}
		}
		else
		{
			for(int32 i = 0; i < MElements.Num(); i++)
			{
				const TVec2<int32>& edge = MElements[i];
				lengths[i] = (InParticles.GetX(edge[0]) - InParticles.GetX(edge[1])).Size();
			}
		}
		return lengths;
	}
}
