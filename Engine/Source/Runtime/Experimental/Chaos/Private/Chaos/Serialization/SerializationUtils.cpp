// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/Serialization/SerializationUtils.h"

#include "Chaos/ConstraintHandle.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/PBDJointConstraints.h"
#include "ChaosVisualDebugger/ChaosVDDataWrapperUtils.h"
#include "DataWrappers/ChaosVDJointDataWrappers.h"
#include "DataWrappers/ChaosVDParticleDataWrapper.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "HAL/Platform.h"

namespace Chaos
{
	namespace Serialization::Private
	{
		static bool bUseFastStructSerializer = false;
		static FAutoConsoleVariableRef CVarSolverSerializerUseFastStructSerializer(TEXT("p.chaos.serialization.UseBuiltInSerializer"), bUseFastStructSerializer, TEXT("If set to true, solver serialization will use the new fast struct serializer. Otherwise it will use the built in serializer"));
	
		static bool CanBulkSerialize(FProperty* Property)
		{
	#if PLATFORM_LITTLE_ENDIAN
			// All numeric properties except TEnumAsByte
			uint64 CastFlags = Property->GetClass()->GetCastFlags();
			if (!!(CastFlags & CASTCLASS_FNumericProperty))
			{
				bool bEnumAsByte = (CastFlags & CASTCLASS_FByteProperty) != 0 && static_cast<FByteProperty*>(Property)->Enum;
				return !bEnumAsByte;
			}
	#endif

			return false;
		}
			
		void SerializeProperty(FProperty* Property, void* PropertyData, FArchive& Ar)
		{
			if (FStructProperty* AsStruct = CastField<FStructProperty>(Property))
			{
				FastStructSerialize(AsStruct->Struct, PropertyData, Ar);
			}
			else if (FArrayProperty* AsArray = CastField<FArrayProperty>(Property))
			{
				if (CanBulkSerialize(AsArray->Inner))
				{
					FScriptArrayHelper ArrayHelper(AsArray, PropertyData);
					int32 ElementCount = 0;		
					if (Ar.IsLoading())
					{
						Ar << ElementCount;
						ArrayHelper.EmptyAndAddUninitializedValues(ElementCount);
					}
					else
					{
						ElementCount = ArrayHelper.Num();
						Ar << ElementCount;
					}
						
					Ar.Serialize(ArrayHelper.GetRawPtr(), AsArray->Inner->GetElementSize() * ElementCount);
				}
				else
				{
					FScriptArrayHelper ArrayHelper(AsArray, PropertyData);
					int32 ElementCount = 0;			
					if (Ar.IsLoading())
					{
						Ar << ElementCount;
						ArrayHelper.EmptyAndAddValues(ElementCount);
					}
					else
					{
						ElementCount = ArrayHelper.Num();
						Ar << ElementCount;
					}

					for (int32 i = 0; i < ElementCount; ++i)
					{
						SerializeProperty(AsArray->Inner, ArrayHelper.GetRawPtr(i), Ar);
					}
				}
			}
			else
			{
				Ar.Serialize(PropertyData, Property->GetElementSize());	
			}
		}

		void FastStructSerialize(UScriptStruct* Struct, void* SourceData, FArchive& Ar, void* Defaults)
		{
			if (!bUseFastStructSerializer)
			{
				Ar.SetUseUnversionedPropertySerialization(true);
				Struct->SerializeTaggedProperties(Ar, static_cast<uint8*>(SourceData), Struct, static_cast<uint8*>(Defaults));
				return;
			}
	
			for (FProperty* Property = Struct->PropertyLink; Property != nullptr; Property = Property->PropertyLinkNext)
			{
				for (int32 Idx = 0; Idx < Property->ArrayDim; Idx++)
				{
					Private::SerializeProperty(Property, Property->ContainerPtrToValuePtr<void>(SourceData, Idx), Ar);
				}
			}
		}
	}
}

