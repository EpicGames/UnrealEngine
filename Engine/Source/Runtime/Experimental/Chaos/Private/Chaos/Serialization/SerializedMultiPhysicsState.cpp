// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/Serialization/SerializedMultiPhysicsState.h"

#include "ChaosLog.h"

namespace Chaos
{
	void FSerializedMultiPhysicsState::ReadElementDataIntoBuffer(FSerializedDataBuffer& TargetBuffer)
	{
		FDataEntryTag DataTag = Header.DataTagPerElementIndex.IsValidIndex(CurrentReadElementIndex) ? Header.DataTagPerElementIndex[CurrentReadElementIndex] : FDataEntryTag();

		TArray<uint8>& AsByteArray = MigratedStateAsBytes.GetDataAsByteArrayRef();

		const int32 BufferSize = GetSize();

		UE_LOGF(LogChaos, Verbose, "[MultiStateSerialization] Attempted to read an empty data entry. This might means we failed to obtain data to serialize a specific body or constraint.");

		// If we have entries past the available buffer size, but they were recorded as empty, the serialization code is ok to continue (it will just be a no-op)
		const bool bHasValidStartingOffset = DataTag.DataOffset < BufferSize || DataTag.DataSize == 0;
		const bool bThereIsEnoughDataToCopyFromBuffer = DataTag.DataOffset + DataTag.DataSize <= BufferSize;

		if (ensure(DataTag.IsValid() && bHasValidStartingOffset && bThereIsEnoughDataToCopyFromBuffer))
		{
			if (DataTag.DataSize > 0)
			{
				TargetBuffer.GetDataAsByteArrayRef().Append(&AsByteArray[DataTag.DataOffset], DataTag.DataSize);
			}
		}
		else
		{
			UE_LOGF(LogChaos, Error, "[MultiStateSerialization] Attempted to read data out of bounds! | Buffer size [%d] | Start Pos [%d]| SizeToCopy [%d]", AsByteArray.Num(), DataTag.DataOffset, DataTag.DataSize);
		}
		
		CurrentReadElementIndex++;
	}


	void FSerializedMultiPhysicsState::Serialize(FArchive& Ar)
	{
		Ar << Header;
		Ar << MigratedStateAsBytes;
	}
}
