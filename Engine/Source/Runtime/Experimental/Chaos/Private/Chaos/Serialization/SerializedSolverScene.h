// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Chaos/ImplicitFwd.h"
#include "DataWrappers/ChaosVDImplicitObjectDataWrapper.h"
#include "SerializedSolverScene.generated.h"

namespace Chaos
{
	class FChaosArchive;
}

struct FChaosVDParticlePairMidPhase;
struct FChaosVDCharacterGroundConstraint;
struct FChaosVDCharacterGroundConstraintDataDataWrapper;
struct FChaosVDJointConstraint;
struct FChaosVDParticleDataWrapper;

using FSerializedImplicitObject = FChaosVDImplicitObjectDataWrapper<Chaos::FImplicitObjectPtr, Chaos::FChaosArchive>;

/**
 * Structure holding the serialized state of a rigid solver
 * Used to save it to disk or to re-hydrate a new solver instance
 */
USTRUCT()
struct UE_INTERNAL FSerializedSolverScene
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FChaosVDParticleDataWrapper> ParticleData;
	UPROPERTY()
	TArray<FChaosVDJointConstraint> JointConstraintData;
	UPROPERTY()
	TArray<FChaosVDCharacterGroundConstraint> CharacterGroundConstraintData;
	UPROPERTY()
	TArray<FChaosVDParticlePairMidPhase> CollisionMidPhaseData;
	
	TArray<FSerializedImplicitObject> ImplicitObjectData;
};