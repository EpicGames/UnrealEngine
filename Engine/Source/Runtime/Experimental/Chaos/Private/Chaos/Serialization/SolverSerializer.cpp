// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/Serialization/SolverSerializer.h"

#include "PBDRigidsSolver.h"
#include "Chaos/PBDJointConstraints.h"
#include "Chaos/Character/CharacterGroundConstraint.h"
#include "Chaos/Character/CharacterGroundConstraintContainer.h"
#include "Chaos/Serialization/SerializationUtils.h"
#include "Chaos/Serialization/SerializedDataBuffer.h"
#include "ChaosVisualDebugger/ChaosVDDataWrapperUtils.h"
#include "DataWrappers/ChaosVDCharacterGroundConstraintDataWrappers.h"
#include "DataWrappers/ChaosVDJointDataWrappers.h"
#include "Serialization/MemoryReader.h"

namespace Chaos
{
	namespace Private
	{
		FChaosVDParticleDataWrapper ExtractParticleDataFromBuffer(FSerializedDataBuffer& InSerializedData)
		{
			FChaosVDParticleDataWrapper ParticleDataWrapper;
	
			using namespace Chaos::Serialization::Private;

			using namespace Chaos::Serialization::Private;
			if (!ensure(!InSerializedData.GetDataAsByteArrayRef().IsEmpty()))
			{
				return MoveTemp(ParticleDataWrapper);
			}

			FMemoryReader MemReader(InSerializedData.GetDataAsByteArrayRef());
			MemReader.SetShouldSkipUpdateCustomVersion(true);
		
			FastStructSerialize(MemReader, &ParticleDataWrapper);

			return MoveTemp(ParticleDataWrapper);
		}

		template<typename ParticleType>
		void WriteParticleDataToBuffer(ParticleType* InParticle, FSerializedDataBuffer& OutSerializedData)
		{
			using namespace Chaos::Serialization::Private;

			if (!ensure(InParticle))
			{
				UE_LOGF(LogChaos, Warning, "[%s]: Failed to serialize particle | Invalid Particle", __func__);
				return;
			}

			// This allows us to serialize multiple particles in a single buffer, if the provided buffer was already used
			constexpr bool bSetOffset = true;
			constexpr bool bIsPersistent = false;

			FMemoryWriter MemWriter(OutSerializedData.GetDataAsByteArrayRef(), bIsPersistent, bSetOffset);
			MemWriter.SetShouldSkipUpdateCustomVersion(true);

			FChaosVDParticleDataWrapper ParticleDataWrapper;

			static_assert(std::is_base_of_v<FGeometryParticleHandle, ParticleType> || std::is_base_of_v<FGeometryParticle, ParticleType>, "WriteParticleDataToBuffer Only supports FGeometryParticleHandle and FGeometryParticle types.");

			// TODO: Handle Geometry and shape data serialization for the case that need it
			if constexpr(std::is_same_v<FGeometryParticleHandle, ParticleType>)
			{
				 ParticleDataWrapper = FChaosVDDataWrapperUtils::BuildParticleDataWrapperFromParticle(InParticle);
			}
			else if (std::is_same_v<FGeometryParticle, ParticleType>)
			{
				ParticleDataWrapper = FChaosVDDataWrapperUtils::BuildParticleDataWrapperFromGTParticle(InParticle);
			}

			FastStructSerialize(MemWriter, &ParticleDataWrapper);
		}

		FChaosVDJointConstraint ExtractJointConstraintDataFromBuffer(FSerializedDataBuffer& InSerializedData)
		{
			FChaosVDJointConstraint JointData;

			if (!ensure(!InSerializedData.GetDataAsByteArrayRef().IsEmpty()))
			{
				return MoveTemp(JointData);
			}
		
			FMemoryReader Reader(InSerializedData.GetDataAsByteArrayRef());
			Reader.SetShouldSkipUpdateCustomVersion(true);
			Serialization::Private::FastStructSerialize(Reader, &JointData);

			return MoveTemp(JointData);
		}
	}

	const TCHAR* LexToString(ESerializedDataContext Value)
	{
		switch (Value)
		{
			case ESerializedDataContext::Internal:
				return TEXT("ESerializedDataContext:Internal");
			case ESerializedDataContext::External:
				return TEXT("ESerializedDataContext:External");
			case ESerializedDataContext::Both:
				return TEXT("ESerializedDataContext:Both");
			case ESerializedDataContext::Invalid:
			default:
				return TEXT("ESerializedDataContext:Invalid");
		}
	}

	FSolverSerializer::FSolverSerializer(FPBDRigidsSolver* InSolver) : SolverInstance(InSolver)
	{
	}

	void FSolverSerializer::SerializeToBuffer(FSerializedDataBuffer& OutSerializedData)
	{
		//TODO: Implement
	}

	void FSolverSerializer::PopulateFromSerializedBuffer(const FSerializedDataBuffer& InSerializedData)
	{
		//TODO: Implement
	}

	void FSolverSerializer::SerializeParticleStateToBuffer(FGeometryParticleHandle* InParticleHandle, FSerializedDataBuffer& OutSerializedData)
	{
		Private::WriteParticleDataToBuffer(InParticleHandle, OutSerializedData);
	}

	void FSolverSerializer::SerializeParticleStateToBuffer(FGeometryParticle* GTParticle, FSerializedDataBuffer& OutSerializedData)
	{
		Private::WriteParticleDataToBuffer(GTParticle, OutSerializedData);
	}

	void FSolverSerializer::SerializeConstraintStateToBuffer(FConstraintHandle* ConstraintHandlePtr, FSerializedDataBuffer& OutSerializedData)
	{
		using namespace Chaos::Serialization::Private;

		if (!ensure(ConstraintHandlePtr))
		{
			UE_LOGF(LogChaos, Warning, "[%s]: Failed to serialize constraint | Invalid Handle", __func__);
			return;
		}

		// This allows us to serialize multiple particles in a single buffer, if the provided buffer was already used
		constexpr bool bSetOffset = true;
		constexpr bool bIsPersistent = false;

		FMemoryWriter MemWriter(OutSerializedData.GetDataAsByteArrayRef(), bIsPersistent, bSetOffset);
		MemWriter.SetShouldSkipUpdateCustomVersion(true);
			
		if (ConstraintHandlePtr->GetType().IsA(FPBDJointConstraintHandle::StaticType()))
		{
			const FPBDJointConstraintHandle* AsJointHandle = static_cast<FPBDJointConstraintHandle*>(ConstraintHandlePtr);	
			FChaosVDJointConstraint JointData = FChaosVDDataWrapperUtils::BuildJointDataWrapper(AsJointHandle);

			FastStructSerialize(MemWriter, &JointData);
		}
		else if (ConstraintHandlePtr->GetType().IsA(FCharacterGroundConstraintHandle::StaticType()))
		{
			const FCharacterGroundConstraintHandle* AsCharacterGroundHandle = static_cast<FCharacterGroundConstraintHandle*>(ConstraintHandlePtr);
			FChaosVDCharacterGroundConstraint CharacterGroundConstraintData = FChaosVDDataWrapperUtils::BuildCharacterGroundConstraintDataWrapper(AsCharacterGroundHandle);
			
			FastStructSerialize(MemWriter, &CharacterGroundConstraintData);
		}
	}


	void FSolverSerializer::SerializeConstraintStateToBuffer(FConstraintBase* GTConstraintPtr, FSerializedDataBuffer& OutSerializedData)
	{
		using namespace Chaos::Serialization::Private;

		if (!ensure(GTConstraintPtr))
		{
			UE_LOGF(LogChaos, Warning, "[%s]: Failed to serialize constraint | Invalid Constraint", __func__);
			return;
		}

		// This allows us to serialize multiple particles in a single buffer, if the provided buffer was already used
		constexpr bool bSetOffset = true;
		constexpr bool bIsPersistent = false;

		FMemoryWriter MemWriter(OutSerializedData.GetDataAsByteArrayRef(), bIsPersistent, bSetOffset);
		MemWriter.SetShouldSkipUpdateCustomVersion(true);
		switch (EConstraintType ConstraintType = GTConstraintPtr->GetType())
		{
			case EConstraintType::JointConstraintType:
				{
					FJointConstraint* AsJoint = static_cast<FJointConstraint*>(GTConstraintPtr);	
					FChaosVDJointConstraint JointData = FChaosVDDataWrapperUtils::BuildGTJointDataWrapper(AsJoint);

					FastStructSerialize(MemWriter, &JointData);
					break;
				}
			case EConstraintType::SpringConstraintType:
			case EConstraintType::SuspensionConstraintType:
			case EConstraintType::CharacterGroundConstraintType:
			case EConstraintType::NoneType:
			default:
				{
					ensureMsgf(false, TEXT("Attempted to Serialize a Constraint type not supported yet | Constraint Type [%d]"), EnumToUnderlyingType(ConstraintType));
					break;
				}
		}
	}


	void FSolverSerializer::ApplySerializedStateToParticle(FGeometryParticleHandle* InParticleHandle, FSerializedDataBuffer& InSerializedData)
	{
		FChaosVDParticleDataWrapper ParticleDataWrapper = Private::ExtractParticleDataFromBuffer(InSerializedData);

		ApplySerializedStateToParticle(InParticleHandle, ParticleDataWrapper);
	}

	void FSolverSerializer::ApplySerializedStateToParticle(FGeometryParticle* GTParticle, FSerializedDataBuffer& InSerializedData)
	{
		FChaosVDParticleDataWrapper ParticleDataWrapper = Private::ExtractParticleDataFromBuffer(InSerializedData);
	
		ApplySerializedStateToParticle(GTParticle, ParticleDataWrapper);
	}

	void FSolverSerializer::ApplySerializedStateToParticle(FGeometryParticle* GTParticle, const FChaosVDParticleDataWrapper& InParticleState)
	{
		if (!ensure(InParticleState.HasValidData() && GTParticle && GTParticle->ObjectType() == static_cast<EParticleType>(InParticleState.Type)))
		{
			return;
		}

		InParticleState.ParticlePositionRotation.CopyTo<FGeometryParticle, FChaosVDParticlePositionRotation::EAccessorType::XR>(*GTParticle);

		if (TKinematicGeometryParticle<FReal, 3>* KinematicParticle = GTParticle->CastToKinematicParticle())
		{
			FKinematicTarget NewKT;
			InParticleState.ParticleKinematicTarget.CopyTo<FKinematicTarget, EKinematicTargetMode>(NewKT);
			KinematicParticle->SetKinematicTarget(NewKT);
			
			InParticleState.ParticleVelocities.CopyTo(*KinematicParticle);
		}

		if (TPBDRigidParticle<FReal, 3>* RigidParticle = GTParticle->CastToRigidParticle())
		{
			InParticleState.ParticleDynamics.CopyTo(*RigidParticle);
			InParticleState.ParticleMassProps.CopyTo(*RigidParticle);
			InParticleState.ParticleDynamicsMisc.CopyWithoutStateTo<TPBDRigidParticle<FReal, 3>, FRigidParticleControlFlags, ESleepType>(*RigidParticle);
			RigidParticle->SetObjectState(static_cast<EObjectStateType>(InParticleState.ParticleDynamicsMisc.MObjectState));
		}

		GTParticle->DirtyFlags().SetBits(InParticleState.DirtyFlagsBits);

		// TODO: Add support for Cluster unions and Geometry Collections
	}

	void FSolverSerializer::ApplySerializedStateToParticle(FGeometryParticleHandle* InParticleHandle, const FChaosVDParticleDataWrapper& InParticleState)
	{
		if (!ensure(InParticleState.HasValidData() && InParticleHandle && InParticleHandle->Type == static_cast<EParticleType>(InParticleState.Type)))
		{
			return;
		}
		
		FPBDRigidsEvolutionGBF& Evolution = *SolverInstance->GetEvolution();

		constexpr bool bIsTeleport = true;
		constexpr bool bWakeUp = false;
		Evolution.SetParticleTransform(InParticleHandle, InParticleState.ParticlePositionRotation.MX, InParticleState.ParticlePositionRotation.MR, bIsTeleport, bWakeUp);

		const FVec3 BoundsExpansion = FVec3(0);
		InParticleHandle->UpdateWorldSpaceState(InParticleHandle->GetTransformXR(), BoundsExpansion);

		Evolution.SetParticleVelocities(InParticleHandle, InParticleState.ParticleVelocities.MV, InParticleState.ParticleVelocities.MW);

		if (TKinematicGeometryParticleHandleImp<FReal, 3, true>* KinematicParticle = InParticleHandle->CastToKinematicParticle())
		{
			FKinematicTarget NewKT;
			InParticleState.ParticleKinematicTarget.CopyTo<FKinematicTarget, EKinematicTargetMode>(NewKT);
			KinematicParticle->SetKinematicTarget(NewKT);
		}

		if (TPBDRigidParticleHandleImp<FReal, 3, true>* RigidParticle = InParticleHandle->CastToRigidParticle())
		{
			InParticleState.ParticleDynamics.CopyTo(*RigidParticle);
			InParticleState.ParticleMassProps.CopyTo(*RigidParticle);
			InParticleState.ParticleDynamicsMisc.CopyWithoutStateTo<TPBDRigidParticleHandleImp<FReal, 3, true>, FRigidParticleControlFlags, ESleepType>(*RigidParticle);
			InParticleState.ParticleVWSmooth.CopyTo<TPBDRigidParticleHandleImp<FReal, 3, true>>(*RigidParticle);

			// Make sure the particle ends up in the correct SoA view
			Evolution.SetParticleObjectState(RigidParticle, static_cast<EObjectStateType>(InParticleState.ParticleDynamicsMisc.MObjectState));
			Evolution.SetParticleSleepType(RigidParticle, RigidParticle->SleepType());
		}

		if (TPBDRigidClusteredParticleHandleImp<FReal, 3, true>* ClusteredParticle = InParticleHandle->CastToClustered())
		{
			InParticleState.ParticleCluster.CopyTo(*ClusteredParticle);
		}

		if (InParticleState.ParticleDynamicsMisc.bDisabled)
		{
			Evolution.DisableParticle(InParticleHandle);
		}
		else
		{
			Evolution.EnableParticle(InParticleHandle);
		}
	}

	void FSolverSerializer::ApplySerializedStateToJointConstraint(FPBDJointConstraintHandle* ConstraintHandlePtr, FSerializedDataBuffer& InSerializedData)
	{
		FChaosVDDataWrapperUtils::ApplyJointDataWrapperToHandle(ConstraintHandlePtr, Private::ExtractJointConstraintDataFromBuffer(InSerializedData));	
	}

	void FSolverSerializer::ApplySerializedStateToJointConstraint(FJointConstraint* ConstraintPtr, FSerializedDataBuffer& InSerializedData)
	{
		FChaosVDDataWrapperUtils::ApplyJointDataWrapperGTConstraint(ConstraintPtr, Private::ExtractJointConstraintDataFromBuffer(InSerializedData));
	}

	void FSolverSerializer::ApplySerializedStateToConstraint(FConstraintHandle* ConstraintHandlePtr, FSerializedDataBuffer& InSerializedData)
	{
		if (!ensure(ConstraintHandlePtr))
		{
			return;
		}

		if (ConstraintHandlePtr->GetType().IsA(FPBDJointConstraintHandle::StaticType()))
		{
			FPBDJointConstraintHandle* AsJointHandle = static_cast<FPBDJointConstraintHandle*>(ConstraintHandlePtr);
			ApplySerializedStateToJointConstraint(AsJointHandle, InSerializedData);
		}
		else
		{
			UE_LOGF(LogChaos, Warning, "Attempted to apply a serialized state to an unsupported constraint type | Type [%ls]. The data will be discarded", *ConstraintHandlePtr->GetType().ToString())
		}
	}

	void FSolverSerializer::ApplySerializedStateToConstraint(FConstraintBase* ConstraintPtr, FSerializedDataBuffer& InSerializedData)
	{
		if (!ConstraintPtr)
		{
			return;
		}

		switch (EConstraintType Type = ConstraintPtr->GetType())
		{
			case EConstraintType::JointConstraintType:
				{
					ApplySerializedStateToJointConstraint(static_cast<FJointConstraint*>(ConstraintPtr), InSerializedData);
					break;
				}
			case EConstraintType::NoneType:
			case EConstraintType::SpringConstraintType:
			case EConstraintType::SuspensionConstraintType:
			case EConstraintType::CharacterGroundConstraintType:
			default:
				{
					ensureMsgf(false, TEXT("Attempted to serialize an unsupported constraint type | Type [%d]"), EnumToUnderlyingType(Type));
					break;
				}
		}
	}

	void FSolverSerializer::PushPendingInternalSerializedStateForProxy(IPhysicsProxyBase* Proxy, FSerializedDataBufferPtr&& InState)
	{
		PendingMigratedPhysicsStateByProxy.Emplace(Proxy, MoveTemp(InState));
	}

	FSerializedDataBufferPtr FSolverSerializer::PopPendingInternalSerializedStateForProxy(IPhysicsProxyBase* Proxy)
	{
		if (PendingMigratedPhysicsStateByProxy.Num() == 0)
		{
			return nullptr;
		}

		FSerializedDataBufferPtr FoundBufferRef = nullptr;
		PendingMigratedPhysicsStateByProxy.RemoveAndCopyValue(Proxy, FoundBufferRef);
		return MoveTemp(FoundBufferRef);
	}
}