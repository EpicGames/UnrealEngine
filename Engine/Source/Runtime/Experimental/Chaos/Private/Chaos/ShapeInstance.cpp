// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/ShapeInstance.h"
#include "Chaos/ImplicitObject.h"
#include "Chaos/ImplicitObjectUnion.h"
#include "Chaos/ImplicitObjectScaled.h"

namespace Chaos
{
	static_assert(sizeof(FShapeInstanceProxy) <= 192, "FShapeInstanceProxy was optimized to fit into 192b bin of MB3 to prevent excess memory waste");

	DECLARE_CYCLE_STAT_WITH_FLAGS(TEXT("UpdateShapesArrayFromGeometryImpl"), STAT_UpdateShapesArrayFromGeometryImpl, STATGROUP_Chaos, EStatFlags::Verbose);
	// Create or reuse the shapes in the shapes array and populate them with the Geometry.
	// If we have a Union it will be unpacked into the ShapesArray.
	// On the Physics Thread we set bAllowCachedLeafInfo which caches the shapes world space state to optimize collision detection,
	// but this flag should not be used on the Game Thread because the extended data is not maintained so some PerShapeData functions
	// will return unitialized values (E.g., GetLeafWorldTransform).
	template<typename TShapesArrayType>
	void UpdateShapesArrayFromGeometryImpl(
		TShapesArrayType& ShapesArray,
		const FImplicitObjectPtr& Geometry,
		const FRigidTransform3& ActorTM,
		IPhysicsProxyBase* Proxy)
	{
		SCOPE_CYCLE_COUNTER(STAT_UpdateShapesArrayFromGeometryImpl);
		// TShapesArrayType = TArray<TUniquePtr<T>>
		using FShapePtrType = typename TShapesArrayType::ElementType;	// TUniquePtr<T>
		using FShapeType = typename FShapePtrType::ElementType;			// T

		if (Geometry)
		{
			const int32 OldShapeNum = ShapesArray.Num();
			if (const FImplicitObjectUnion* Union = Geometry->template GetObject<FImplicitObjectUnion>())
			{
				const int32 NumShapes = Union->GetObjects().Num();
				ShapesArray.SetNum(NumShapes);
				
				for (int32 ShapeIndex = 0; ShapeIndex < NumShapes; ++ShapeIndex)
				{
					FImplicitObjectPtr ShapeGeometry = Union->GetObjects()[ShapeIndex];
					
					if (ShapeIndex >= OldShapeNum)
					{
						// If newly allocated shape, initialize it.
						ShapesArray[ShapeIndex] = FShapeType::Make(ShapeIndex, ShapeGeometry);
					}
					else if (ShapeGeometry != ShapesArray[ShapeIndex]->GetGeometry())
					{
						// Update geometry pointer if it changed
						FShapeType::UpdateGeometry(ShapesArray[ShapeIndex], ShapeGeometry);
						ShapesArray[ShapeIndex]->ModifyShapeIndex(ShapeIndex);
					}
				}
			}
			else
			{
				ShapesArray.Reserve(1);
				ShapesArray.SetNum(1);
				if (OldShapeNum == 0)
				{
					ShapesArray[0] = FShapeType::Make(0, Geometry);
				}
				else
				{
					FShapeType::UpdateGeometry(ShapesArray[0], Geometry);
				}
			}

			if (Geometry->HasBoundingBox())
			{
				for (auto& Shape : ShapesArray)
				{
					Shape->UpdateShapeBounds(ActorTM);
				}
			}
		}
		else
		{
			ShapesArray.Reset();
		}

		if (Proxy)
		{
			if (FPhysicsSolverBase* PhysicsSolverBase = Proxy->GetSolver<FPhysicsSolverBase>())
			{
				PhysicsSolverBase->SetNumDirtyShapes(Proxy, ShapesArray.Num());
			}
		}
	}

	void UpdateShapesArrayFromGeometry(
		FShapeInstanceArray& ShapesArray, 
		const FImplicitObjectPtr& Geometry, 
		const FRigidTransform3& ActorTM)
	{
		UpdateShapesArrayFromGeometryImpl(ShapesArray, Geometry, ActorTM, nullptr);
	}

	void UpdateShapesArrayFromGeometry(
		FShapeInstanceProxyArray& ShapesArray, 
		const FImplicitObjectPtr& Geometry, 
		const FRigidTransform3& ActorTM, 
		IPhysicsProxyBase* Proxy)
	{
		UpdateShapesArrayFromGeometryImpl(ShapesArray, Geometry, ActorTM, Proxy);
	}

	void UpdateShapesArrayFromGeometry(
		FShapesArray& ShapesArray,
		TSerializablePtr<FImplicitObject> Geometry,
		const FRigidTransform3& ActorTM,
		IPhysicsProxyBase* Proxy)
	{
		check(false);
	}

	// Unwrap transformed shapes
	// @todo(chaos): also unwrap Instanced and Scaled but that requires a lot of knock work because Convexes are usually Instanced or
	// Scaled so the Scale and Margin is stored on the wrapper (the convex itself is shared).
	// - support for Margin as per-shape data passed through the collision functions
	// - support for Scale as per-shape data passed through the collision functions (or ideally in the Transforms)
	const FImplicitObject* GetInnerGeometryInstanceData(const FImplicitObject* Implicit, const FRigidTransform3** OutRelativeTransformPtr)
	{
		if (Implicit != nullptr)
		{
			const EImplicitObjectType ImplicitOuterType = Implicit->GetType();
			if (ImplicitOuterType == TImplicitObjectTransformed<FReal, 3>::StaticType())
			{
				// Transformed Implicit
				const TImplicitObjectTransformed<FReal, 3>* TransformedImplicit = Implicit->template GetObject<const TImplicitObjectTransformed<FReal, 3>>();
				if (OutRelativeTransformPtr != nullptr)
				{
					*OutRelativeTransformPtr = &TransformedImplicit->GetTransform();
				}
				return GetInnerGeometryInstanceData(TransformedImplicit->GetTransformedObject(), OutRelativeTransformPtr);
			}
			else if ((uint32)ImplicitOuterType & ImplicitObjectType::IsInstanced)
			{
				// Instanced Implicit
				// Currently we only unwrap instanced TriMesh and Heightfields. Convex is the only other type that will be Instanced,
				// but we do not unwrap Convex because the convex margin is stored in the Instanced so we would need to extract
				// it and pass it on to the collision detection.
				const FImplicitObjectInstanced* Instanced = static_cast<const FImplicitObjectInstanced*>(Implicit);
				EImplicitObjectType InnerType = Instanced->GetInnerObject()->GetType();
				if (InnerType != FImplicitConvex3::StaticType())
				{
					return GetInnerGeometryInstanceData(Instanced->GetInnerObject().Get(), OutRelativeTransformPtr);
				}
			}
			else if ((ImplicitOuterType == FImplicitObjectUnion::StaticType()) || (ImplicitOuterType == FImplicitObjectUnionClustered::StaticType()))
			{
				// If the union only has one child, we keep recursing, otherwise we don't
				const FImplicitObjectUnion* Union = static_cast<const FImplicitObjectUnion*>(Implicit);
				if (Union->GetObjects().Num() == 1)
				{
					return GetInnerGeometryInstanceData(Union->GetObjects()[0].GetReference(), OutRelativeTransformPtr);
				}
			}
			else if ((uint32)ImplicitOuterType & ImplicitObjectType::IsScaled)
			{
				// Scaled Implicit
				//const FImplicitObjectScaled* Scaled = static_cast<const FImplicitObjectScaled*>(Implicit);
				//OutTransform.Scale *= Scaled->GetScale();
				//return GetInnerGeometryInstanceData(Scaled->GetInnerObject().Get(), OutRelativeTransformPtr);
			}
		}
		return Implicit;
	}

	bool ShapeInstanceWantsLeafCache(const FImplicitObject* Geometry)
	{
		// We will cache the world space transform if we are wrapped in a transform implicit
		const FRigidTransform3* LeafRelativeTransform = nullptr;
		const FImplicitObject* LeafGeometry = GetInnerGeometryInstanceData(Geometry, &LeafRelativeTransform);
		return (LeafGeometry != Geometry) && (LeafRelativeTransform != nullptr);
	}

	///////////////////////////////////////////////////////////////////////////////////////////////
	///////////////////////////////////////////////////////////////////////////////////////////////
	// 
	// Factory methods
	// 
	///////////////////////////////////////////////////////////////////////////////////////////////
	///////////////////////////////////////////////////////////////////////////////////////////////

	TUniquePtr<FPerShapeData> FPerShapeData::CreatePerShapeData(int32 InShapeIdx, TSerializablePtr<FImplicitObject> InGeometry)
	{
		check(false);
		return nullptr;
	}

	void FPerShapeData::UpdateGeometry(TUniquePtr<FPerShapeData>& InOutShapePtr, TSerializablePtr<FImplicitObject> InGeometry)
	{
		check(false);
	}

	const Filter::FShapeFilterData FPerShapeData::GetShapeFilterData() const
	{
		return DownCast([](const auto& ShapeInstance) -> const auto { return ShapeInstance.GetShapeFilterData(); });
	}

	void FPerShapeData::SetShapeFilterData(const Filter::FShapeFilterData& InShapeFilter)
	{
		return DownCast([&InShapeFilter](auto& ShapeInstance) { ShapeInstance.SetShapeFilterData(InShapeFilter); });
	}

	const Filter::FInstanceData FPerShapeData::GetFilterInstanceData() const
	{
		return DownCast([](const auto& ShapeInstance) -> const auto { return ShapeInstance.GetFilterInstanceData(); });
	}

	void FPerShapeData::SetFilterInstanceData(const Filter::FInstanceData& InInstanceData)
	{
		return DownCast([&InInstanceData](auto& ShapeInstance) { ShapeInstance.SetFilterInstanceData(InInstanceData); });
	}

	const Filter::FCombinedShapeFilterData FPerShapeData::GetCombinedShapeFilterData() const
	{
		return DownCast([](const auto& ShapeInstance) -> const auto { return ShapeInstance.GetCombinedShapeFilterData(); });
	}

	void FPerShapeData::SetCombinedShapeFilterData(const Filter::FCombinedShapeFilterData& InShapeCombinedFilter)
	{
		return DownCast([&InShapeCombinedFilter](auto& ShapeInstance) { ShapeInstance.SetCombinedShapeFilterData(InShapeCombinedFilter); });
	}

	TUniquePtr<FShapeInstanceProxy> FShapeInstanceProxy::Make(int32 InShapeIdx, const FImplicitObjectPtr& InGeometry)
	{
		return TUniquePtr<FShapeInstanceProxy>(new FShapeInstanceProxy(InShapeIdx, InGeometry));
	}

	void FShapeInstanceProxy::UpdateGeometry(TUniquePtr<FShapeInstanceProxy>& InOutShapePtr, const FImplicitObjectPtr& InGeometry)
	{
		InOutShapePtr->Geometry = InGeometry;
	}

	const Filter::FShapeFilterData FShapeInstanceProxy::GetShapeFilterData() const
	{
		return CollisionData.Read().GetShapeFilterData();
	}

	void FShapeInstanceProxy::SetShapeFilterData(const Filter::FShapeFilterData& InShapeFilter)
	{
		CollisionData.Modify(true, DirtyFlags, Proxy, ShapeIdx, [InShapeFilter](FCollisionData& Data) { Data.SetShapeFilterData(InShapeFilter); });
	}

	const Filter::FInstanceData FShapeInstanceProxy::GetFilterInstanceData() const
	{
		return CollisionData.Read().GetFilterInstanceData();
	}

	void FShapeInstanceProxy::SetFilterInstanceData(const Filter::FInstanceData& InInstanceData)
	{
		CollisionData.Modify(true, DirtyFlags, Proxy, ShapeIdx, [InInstanceData](FCollisionData& Data) { Data.SetFilterInstanceData(InInstanceData); });
	}

	const Filter::FCombinedShapeFilterData FShapeInstanceProxy::GetCombinedShapeFilterData() const
	{
		return CollisionData.Read().GetCombinedShapeFilterData();
	}

	void FShapeInstanceProxy::SetCombinedShapeFilterData(const Filter::FCombinedShapeFilterData& InShapeCombinedFilter)
	{
		CollisionData.Modify(true, DirtyFlags, Proxy, ShapeIdx, [InShapeCombinedFilter](FCollisionData& Data) { Data.SetCombinedShapeFilterData(InShapeCombinedFilter); });
	}


	TUniquePtr<FShapeInstance> FShapeInstance::Make(int32 InShapeIdx, const FImplicitObjectPtr& InGeometry)
	{
		const bool bWantLeafCache = ShapeInstanceWantsLeafCache(InGeometry.GetReference());
		if (bWantLeafCache)
		{
			return TUniquePtr<FShapeInstance>(new Private::FShapeInstanceExtended(InShapeIdx, InGeometry));
		}
		else
		{
			return TUniquePtr<FShapeInstance>(new FShapeInstance(InShapeIdx, InGeometry));
		}
	}

	void FShapeInstance::UpdateGeometry(TUniquePtr<FShapeInstance>& InOutShapePtr, const FImplicitObjectPtr& InGeometry)
	{
		const bool bWantLeafCache = ShapeInstanceWantsLeafCache(InGeometry.GetReference());

		// Do we need to add or remove the cached leaf data? If so this requires we recreate the object
		const bool bHasLeafCache = (InOutShapePtr->GetType() == EPerShapeDataType::SimExtended);
		if (bWantLeafCache != bHasLeafCache)
		{
			FShapeInstance* ShapeInstance = InOutShapePtr.Get();
			if (bWantLeafCache)
			{
				InOutShapePtr = TUniquePtr<FShapeInstance>(new Private::FShapeInstanceExtended(MoveTemp(*ShapeInstance)));
			}
			else
			{
				InOutShapePtr = TUniquePtr<FShapeInstance>(new FShapeInstance(MoveTemp(*ShapeInstance)));
			}
		}

		InOutShapePtr->Geometry = InGeometry;
	}

	///////////////////////////////////////////////////////////////////////////////////////////////
	///////////////////////////////////////////////////////////////////////////////////////////////
	// 
	// Serialization
	// 
	///////////////////////////////////////////////////////////////////////////////////////////////
	///////////////////////////////////////////////////////////////////////////////////////////////

	void FPerShapeData::Serialize(FChaosArchive& Ar)
	{
		Ar.UsingCustomVersion(FExternalPhysicsCustomObjectVersion::GUID);
		Ar.UsingCustomVersion(FExternalPhysicsMaterialCustomObjectVersion::GUID);

		Ar << Geometry;

		DownCast([&Ar](auto& ShapeInstance)
			{
				Ar << ShapeInstance.CollisionData;
			});

		SerializeMaterials(Ar);

		if (Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) >= FExternalPhysicsCustomObjectVersion::SerializeShapeWorldSpaceBounds)
		{
			TBox<FReal, 3>::SerializeAsAABB(Ar, WorldSpaceShapeBounds);
		}
		else
		{
			// This should be set by particle serializing this FPerShapeData.
			WorldSpaceShapeBounds = FAABB3(FVec3(0.0f, 0.0f, 0.0f), FVec3(0.0f, 0.0f, 0.0f));
		}
	}

	FPerShapeData* FPerShapeData::SerializationFactory(FChaosArchive& Ar, FPerShapeData*)
	{
		// We should not be serializing base class pointers - should always be FShapeInstanceProxy or FShapeInstance
		ensure(false);
		return FShapeInstanceProxy::SerializationFactory(Ar, nullptr);
	}

	FShapeInstanceProxy* FShapeInstanceProxy::SerializationFactory(FChaosArchive& Ar, FShapeInstanceProxy*)
	{
		//todo: need to rework serialization for shapes, for now just give them all shape idx 0
		return Ar.IsLoading() ? new FShapeInstanceProxy(0) : nullptr;
	}

	void FShapeInstanceProxy::SerializeMaterials(FChaosArchive& Ar)
	{
		Ar << Materials;
	}

	FShapeInstance* FShapeInstance::SerializationFactory(FChaosArchive& Ar, FShapeInstance*)
	{
		//todo: need to rework serialization for shapes, for now just give them all shape idx 0
		return Ar.IsLoading() ? new FShapeInstance(0) : nullptr;
	}

	void FShapeInstance::SerializeMaterials(FChaosArchive& Ar)
	{
		// NOTE: FShapeInstanceProxy and FShapeInstance used to be serialized as FPerShapeData
		// so we must ensure what we serialize here is binary compatible with FShapeInstanceProxy::SerializeMaterials
		if (Ar.IsLoading())
		{
			FMaterialData Data;
			Ar << Data;
			SetMaterialData(Data);
		}
		else
		{
			if (bIsSingleMaterial)
			{
				FMaterialData Data;
				Data.Materials.Add(Material.MaterialHandle);
				Ar << Data;
			}
			else
			{
				Ar << GetMaterialDataImpl();
			}
		}
	}

	const Filter::FShapeFilterData FShapeInstance::GetShapeFilterData() const
	{
		return CollisionData.GetShapeFilterData();
	}

	void FShapeInstance::SetShapeFilterData(const Filter::FShapeFilterData& InShapeFilter)
	{
		CollisionData.SetShapeFilterData(InShapeFilter);
	}

	const Filter::FInstanceData FShapeInstance::GetFilterInstanceData() const
	{
		return CollisionData.GetFilterInstanceData();
	}

	void FShapeInstance::SetFilterInstanceData(const Filter::FInstanceData& InInstanceData)
	{
		CollisionData.SetFilterInstanceData(InInstanceData);
	}

	const Filter::FCombinedShapeFilterData FShapeInstance::GetCombinedShapeFilterData() const
	{
		return CollisionData.GetCombinedShapeFilterData();
	}

	void FShapeInstance::SetCombinedShapeFilterData(const Filter::FCombinedShapeFilterData& InCombinedShapeFilter)
	{
		CollisionData.SetCombinedShapeFilterData(InCombinedShapeFilter);
	}

	///////////////////////////////////////////////////////////////////////////////////////////////
	///////////////////////////////////////////////////////////////////////////////////////////////
	// 
	// FShapeInstanceProxy
	// 
	///////////////////////////////////////////////////////////////////////////////////////////////
	///////////////////////////////////////////////////////////////////////////////////////////////


	void FShapeInstanceProxy::UpdateShapeBounds(const FRigidTransform3& WorldTransform)
	{
		if (Geometry && Geometry->HasBoundingBox())
		{
			WorldSpaceShapeBounds = Geometry->CalculateTransformedBounds(WorldTransform);
		}
		else
		{
			WorldSpaceShapeBounds = FAABB3(WorldTransform.GetLocation(), WorldTransform.GetLocation());
		}
	}

	void FShapeInstanceProxy::UpdateWorldSpaceState(const FRigidTransform3& WorldTransform)
	{
		UpdateShapeBounds(WorldTransform);
	}

	const FImplicitObject* FShapeInstanceProxy::GetLeafGeometry() const
	{
		return GetInnerGeometryInstanceData(Geometry.GetReference(), nullptr);
	}

	FRigidTransform3 FShapeInstanceProxy::GetLeafRelativeTransform() const
	{
		const FRigidTransform3* LeafRelativeTransform = nullptr;
		GetInnerGeometryInstanceData(Geometry.GetReference(), &LeafRelativeTransform);

		if (LeafRelativeTransform != nullptr)
		{
			return *LeafRelativeTransform;
		}
		else
		{
			return FRigidTransform3::Identity;
		}
	}

	FRigidTransform3 FShapeInstanceProxy::GetLeafWorldTransform(const FGeometryParticleHandle* Particle) const
	{
		FRigidTransform3 LeafWorldTransform = FConstGenericParticleHandle(Particle)->GetTransformPQ();

		const FRigidTransform3* LeafRelativeTransform = nullptr;
		GetInnerGeometryInstanceData(Geometry.GetReference(), &LeafRelativeTransform);
		if (LeafRelativeTransform != nullptr)
		{
			LeafWorldTransform = FRigidTransform3::MultiplyNoScale(*LeafRelativeTransform, LeafWorldTransform);
		}

		return LeafWorldTransform;
	}

	void FShapeInstanceProxy::UpdateLeafWorldTransform(FGeometryParticleHandle* Particle)
	{
	}

	///////////////////////////////////////////////////////////////////////////////////////////////
	///////////////////////////////////////////////////////////////////////////////////////////////
	// 
	// FShapeInstance
	// 
	///////////////////////////////////////////////////////////////////////////////////////////////
	///////////////////////////////////////////////////////////////////////////////////////////////


	void FShapeInstance::UpdateShapeBounds(const FRigidTransform3& WorldTransform)
	{
		if (Geometry && Geometry->HasBoundingBox())
		{
			WorldSpaceShapeBounds = Geometry->CalculateTransformedBounds(WorldTransform);
		}
		else
		{
			WorldSpaceShapeBounds = FAABB3(WorldTransform.GetLocation(), WorldTransform.GetLocation());
		}
	}

	void FShapeInstance::UpdateWorldSpaceState(const FRigidTransform3& WorldTransform)
	{
		FRigidTransform3 LeafWorldTransform = WorldTransform;
		const FRigidTransform3* LeafRelativeTransform = nullptr;
		const FImplicitObject* LeafGeometry = GetInnerGeometryInstanceData(Geometry.GetReference(), &LeafRelativeTransform);

		// Calculate the leaf world transform if different from particle transform
		if (LeafRelativeTransform != nullptr)
		{
			LeafWorldTransform = FRigidTransform3::MultiplyNoScale(*LeafRelativeTransform, WorldTransform);
		}

		// Store the leaf data if we have a cache
		if (Private::FShapeInstanceExtended* LeafCache = AsShapeInstanceExtended())
		{
			LeafCache->SetWorldTransform(LeafWorldTransform);
		}

		// Update the bounds at the world transform
		if ((LeafGeometry != nullptr) && LeafGeometry->HasBoundingBox())
		{
			WorldSpaceShapeBounds = LeafGeometry->CalculateTransformedBounds(LeafWorldTransform);
		}
		else
		{
			WorldSpaceShapeBounds = FAABB3(WorldTransform.GetLocation(), WorldTransform.GetLocation());
		}
	}

	const FImplicitObject* FShapeInstance::GetLeafGeometry() const
	{
		return GetInnerGeometryInstanceData(Geometry.GetReference(), nullptr);
	}

	FRigidTransform3 FShapeInstance::GetLeafRelativeTransform() const
	{
		const FRigidTransform3* LeafRelativeTransform = nullptr;
		GetInnerGeometryInstanceData(Geometry.GetReference(), &LeafRelativeTransform);

		if (LeafRelativeTransform != nullptr)
		{
			return *LeafRelativeTransform;
		}
		else
		{
			return FRigidTransform3::Identity;
		}
	}

	FRigidTransform3 FShapeInstance::GetLeafWorldTransform(const FGeometryParticleHandle* Particle) const
	{
		if (const Private::FShapeInstanceExtended* LeafCache = AsShapeInstanceExtended())
		{
			return LeafCache->GetWorldTransform();
		}
		else
		{
			FRigidTransform3 LeafWorldTransform = FConstGenericParticleHandle(Particle)->GetTransformPQ();

			const FRigidTransform3* LeafRelativeTransform = nullptr;
			GetInnerGeometryInstanceData(Geometry.GetReference(), &LeafRelativeTransform);
			if (LeafRelativeTransform != nullptr)
			{
				LeafWorldTransform = FRigidTransform3::MultiplyNoScale(*LeafRelativeTransform, LeafWorldTransform);
			}

			return LeafWorldTransform;
		}
	}

	void FShapeInstance::UpdateLeafWorldTransform(FGeometryParticleHandle* Particle)
	{
		if (Private::FShapeInstanceExtended* LeafCache = AsShapeInstanceExtended())
		{
			FRigidTransform3 LeafWorldTransform = FConstGenericParticleHandle(Particle)->GetTransformPQ();

			const FRigidTransform3* LeafRelativeTransform = nullptr;
			GetInnerGeometryInstanceData(Geometry.GetReference(), &LeafRelativeTransform);
			if (LeafRelativeTransform != nullptr)
			{
				LeafWorldTransform = FRigidTransform3::MultiplyNoScale(*LeafRelativeTransform, LeafWorldTransform);
			}

			LeafCache->SetWorldTransform(LeafWorldTransform);
		}
	}

}