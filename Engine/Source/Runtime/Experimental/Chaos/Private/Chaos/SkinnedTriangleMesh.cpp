// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/SkinnedTriangleMesh.h"
#include "Chaos/ImplicitObjectScaled.h"


namespace Chaos
{
	FSkinnedTriangleMesh::FSkinnedTriangleMesh(FTriangleMesh&& InTriangleMesh, TArray<FVec3f>&& InReferencePositions, TArray<FWeightedInfluenceData>&& InBoneData, 
		TArray<FName>&& InUsedBones, FTransform&& InReferenceRootTransform, TArray<FTransform>&& InReferenceRelativeTransforms)
		: FImplicitObject(EImplicitObject::HasBoundingBox, ImplicitObjectType::SkinnedTriangleMesh)
		, TriangleMesh(MoveTemp(InTriangleMesh))
		, BoneData(MoveTemp(InBoneData))
		, UsedBones(MoveTemp(InUsedBones))
		, ReferenceRootTransform(MoveTemp(InReferenceRootTransform))
		, ReferenceRelativeTransforms(MoveTemp(InReferenceRelativeTransforms))
		, ReferencePositions(MoveTemp(InReferencePositions))
	{
		ReferenceBoundingBox = CalculateBoundingBox(TConstArrayView<FVec3f>(ReferencePositions));
		FinalizeConstruction();
	}

	FSkinnedTriangleMesh::FSkinnedTriangleMesh(FSkinnedTriangleMesh&& Other)
		: FImplicitObject(EImplicitObject::HasBoundingBox, ImplicitObjectType::SkinnedTriangleMesh)
		, TriangleMesh(MoveTemp(Other.TriangleMesh))
		, BoneData(MoveTemp(Other.BoneData))
		, UsedBones(MoveTemp(Other.UsedBones))
		, ReferenceRootTransform(MoveTemp(Other.ReferenceRootTransform))
		, ReferenceRelativeTransforms(MoveTemp(Other.ReferenceRelativeTransforms))
		, ReferencePositions(MoveTemp(Other.ReferencePositions))
		, ReferenceBoundingBox(MoveTemp(Other.ReferenceBoundingBox))
		, LocalBoundingBox(MoveTemp(Other.LocalBoundingBox))
		, LocalPositions(MoveTemp(Other.LocalPositions))
	{
	}

	TAABB<FRealSingle, 3> FSkinnedTriangleMesh::CalculateBoundingBox(const TConstArrayView<FVec3f>& Positions) const
	{
		// Should we calculate based on only positions used by Elements, or should we require that Positions contains no extra points?
		TAABB<FRealSingle, 3> AABB;

		for (const TVec3<int32>& Element : TriangleMesh.GetElements())
		{
			AABB.GrowToInclude(Positions[Element[0]]);
			AABB.GrowToInclude(Positions[Element[1]]);
			AABB.GrowToInclude(Positions[Element[2]]);
		}
		
		return AABB;
	}

	void FSkinnedTriangleMesh::FinalizeConstruction()
	{
		LocalPositions = ReferencePositions;
		LocalBoundingBox = ReferenceBoundingBox;
	}

	FImplicitObjectPtr FSkinnedTriangleMesh::CopyGeometry() const
	{
		FSkinnedTriangleMesh* const Copy = new FSkinnedTriangleMesh();
		constexpr bool bCullDegenerateElementsFalse = false;
		Copy->TriangleMesh.Init(TriangleMesh.GetElements(), TriangleMesh.GetStartIndex(), TriangleMesh.GetNumIndices(), bCullDegenerateElementsFalse);
		Copy->BoneData = BoneData;
		Copy->UsedBones = UsedBones;
		Copy->ReferenceRootTransform = ReferenceRootTransform;
		Copy->ReferenceRelativeTransforms = ReferenceRelativeTransforms;
		Copy->ReferencePositions = ReferencePositions;
		Copy->ReferenceBoundingBox = ReferenceBoundingBox;
		Copy->LocalPositions = LocalPositions;
		Copy->LocalBoundingBox = LocalBoundingBox;
		return FImplicitObjectPtr(Copy);
	}

	FImplicitObjectPtr FSkinnedTriangleMesh::CopyGeometryWithScale(const FVec3& Scale) const
	{
		FSkinnedTriangleMesh* const Copy = new FSkinnedTriangleMesh();
		constexpr bool bCullDegenerateElementsFalse = false;
		Copy->TriangleMesh.Init(TriangleMesh.GetElements(), TriangleMesh.GetStartIndex(), TriangleMesh.GetNumIndices(), bCullDegenerateElementsFalse);
		Copy->BoneData = BoneData;
		Copy->UsedBones = UsedBones;
		Copy->ReferenceRootTransform = ReferenceRootTransform;
		Copy->ReferenceRelativeTransforms = ReferenceRelativeTransforms;
		Copy->ReferencePositions = ReferencePositions;
		Copy->ReferenceBoundingBox = ReferenceBoundingBox;
		Copy->LocalPositions = LocalPositions;
		Copy->LocalBoundingBox = LocalBoundingBox;
		return MakeImplicitObjectPtr<TImplicitObjectScaled<FSkinnedTriangleMesh>>(Copy, Scale);
	}

	uint32 FSkinnedTriangleMesh::GetTypeHash() const
	{
		uint32 Result = GetArrayHash(TriangleMesh.GetElements().GetData(), TriangleMesh.GetElements().Num());
		for (const FWeightedInfluenceData& Data : BoneData)
		{
			Result = HashCombine(Result, Data.GetTypeHash());
		}
		Result = GetArrayHash(ReferencePositions.GetData(), ReferencePositions.Num(), Result);
		return Result;
	}

	void FSkinnedTriangleMesh::SkinPositions(const TArray<FTransform>& RelativeTransforms, const TArrayView<FVec3f>& Positions) const
	{
		check(Positions.Num() == BoneData.Num());
		check(RelativeTransforms.Num() == ReferenceRelativeTransforms.Num());
		TArray<FTransform3f> BoneTransforms;
		BoneTransforms.SetNum(RelativeTransforms.Num());
		for (int32 Index = 0; Index < RelativeTransforms.Num(); ++Index)
		{
			BoneTransforms[Index] = FTransform3f(ReferenceRelativeTransforms[Index] * RelativeTransforms[Index]);
		}

		// TODO: ISPC version

		auto AddInfluence = [&BoneTransforms](FVec3f& OutPosition, const FVec3f& RefPosition, const uint16 BoneIndex, const float BoneWeight)
			{
				OutPosition += BoneTransforms[BoneIndex].TransformPosition(RefPosition) * BoneWeight;
			};

		for (int32 VertIndex = 0; VertIndex < BoneData.Num(); ++VertIndex)
		{
			// This is taken from ChaosClothingSimulationMesh although it uses FMatrix44f matrices instead of FTransform3f.
			const uint16* const RESTRICT BoneIndices = BoneData[VertIndex].BoneIndices;
			const float* const RESTRICT BoneWeights = BoneData[VertIndex].BoneWeights;
			// WARNING - HORRIBLE UNROLLED LOOP + JUMP TABLE BELOW
			// done this way because this is a pretty tight and performance critical loop. essentially
			// rather than checking each influence we can just jump into this switch and fall through
			// everything to compose the final skinned data

			const FVec3f& RefPosition = ReferencePositions[VertIndex];
			FVec3f Position((FRealSingle)0.f);

			switch (BoneData[VertIndex].NumInfluences)
			{
			default: // Intentional fallthrough
			case 12: AddInfluence(Position, RefPosition, BoneIndices[11], BoneWeights[11]); // Intentional fallthrough
			case 11: AddInfluence(Position, RefPosition, BoneIndices[10], BoneWeights[10]); // Intentional fallthrough
			case 10: AddInfluence(Position, RefPosition, BoneIndices[ 9], BoneWeights[ 9]); // Intentional fallthrough
			case  9: AddInfluence(Position, RefPosition, BoneIndices[ 8], BoneWeights[ 8]); // Intentional fallthrough
			case  8: AddInfluence(Position, RefPosition, BoneIndices[ 7], BoneWeights[ 7]); // Intentional fallthrough
			case  7: AddInfluence(Position, RefPosition, BoneIndices[ 6], BoneWeights[ 6]); // Intentional fallthrough
			case  6: AddInfluence(Position, RefPosition, BoneIndices[ 5], BoneWeights[ 5]); // Intentional fallthrough
			case  5: AddInfluence(Position, RefPosition, BoneIndices[ 4], BoneWeights[ 4]); // Intentional fallthrough
			case  4: AddInfluence(Position, RefPosition, BoneIndices[ 3], BoneWeights[ 3]); // Intentional fallthrough
			case  3: AddInfluence(Position, RefPosition, BoneIndices[ 2], BoneWeights[ 2]); // Intentional fallthrough
			case  2: AddInfluence(Position, RefPosition, BoneIndices[ 1], BoneWeights[ 1]); // Intentional fallthrough
			case  1: AddInfluence(Position, RefPosition, BoneIndices[ 0], BoneWeights[ 0]); // Intentional fallthrough
			case  0: break;
			}
			Positions[VertIndex] = Position;
		}
	}

	void FSkinnedTriangleMesh::UpdateSpatialHierarchy(const FRealSingle MinLodSize)
	{		
		TriangleMesh.BuildSpatialHash(TConstArrayView<FVec3f>(LocalPositions), SpatialHash, MinLodSize);
	}
}
