// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/SmokeEvolution.h"

#include "Chaos/Framework/Parallel.h"
#include "Chaos/PerCellBoundaryConditions.h"

namespace Chaos
{
	template<class T, int d>
	void TSmokeEvolution<T, d>::AdvanceOneTimeStep(const T Dt)
	{
		TPerCellBoundaryConditions<T, d> BoundaryRule(MCollisionParticles, MSourceParticles);

		TArrayFaceND<T, d> VelocityN = MVelocity.Copy();
		TArrayND<T, d> DensityN = MDensity.Copy();
		PhysicsParallelFor(MGrid.GetNumCells(), [&](int32 Index)
		{
			auto CellIndex = MGrid.GetIndex(Index);
			MAdvectionRule(MGrid, MDensity, DensityN, VelocityN, Dt, CellIndex);
		});
		for(int32 i = 0; i < d; ++i)
		{
			TVector<T, d> HalfDx = TVector<T, d>::AxisVector(i) * (MGrid.Dx()[i] / 2);
			TUniformGrid<T, d> DualGrid(MGrid.MinCorner() - HalfDx, MGrid.MaxCorner() + HalfDx, MGrid.Counts() + TVector<int32, d>::AxisVector(i));
			PhysicsParallelFor(DualGrid.GetNumCells(), [&](int32 Index)
			{
				auto CellIndex = DualGrid.GetIndex(Index);
				MConvectionRule(DualGrid, MVelocity.GetComponent(i), VelocityN.GetComponent(i), VelocityN, Dt, CellIndex);
			});
		}
		PhysicsParallelFor(MGrid.GetNumFaces(), [&](int32 Index)
		{
			auto FaceIndex = MGrid.GetFaceIndex(Index);
			for(auto ForceRule : MForceRules)
			{
				ForceRule(MGrid, MVelocity, Dt, FaceIndex);
			}
			BoundaryRule.ApplyNeumann(MGrid, MNeumann, MVelocity, Dt, FaceIndex);
		});
		PhysicsParallelFor(MGrid.GetNumCells(), [&](int32 Index)
		{
			auto CellIndex = MGrid.GetIndex(Index);
			BoundaryRule.ApplyDirichlet(MGrid, MDirichlet, MDensity, Dt, CellIndex);
		});
		MProjectionRule(MGrid, MVelocity, MDirichlet, MNeumann, Dt);
	}
}

template class Chaos::TSmokeEvolution<Chaos::FReal, 3>;
