// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/SoftsEvolution.h"
#include "Chaos/SoftsEvolutionLinearSystem.h"
#include "Chaos/PBDFlatWeightMap.h"

namespace Chaos::Softs {

namespace Private
{
namespace EvolutionSolverDefault
{
constexpr FSolverReal SolverFrequency = (FSolverReal)60.;
constexpr int32 NumIterations = 1;
constexpr int32 MinNumIterations = 1;
constexpr int32 MaxNumIterations = 10;
constexpr bool bEnableForceBasedSolver = false;
constexpr int32 NumNewtonIterations = 1;
constexpr bool bDoQuasistatics = false;
}
namespace EvolutionSoftBodyDefault
{
constexpr FSolverReal GlobalDamping = (FSolverReal)0.01;
constexpr FSolverReal LocalDamping = (FSolverReal)0.;
constexpr EChaosSoftsLocalDampingSpace LocalDampingSpace = EChaosSoftsLocalDampingSpace::CenterOfMass;
constexpr bool bUsePerParticleDamping = false;
}

static void EulerStepVelocity(FSolverParticlesRange& Particles, const FSolverReal Dt)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSoftsEvolution_EulerStepVelocity);
	// TODO: ISPC and/or ParallelFor
	FSolverVec3* const Velocity = Particles.GetV().GetData();
	const FSolverVec3* const Acceleration = Particles.GetAcceleration().GetData();
	const FSolverReal* const InvM = Particles.GetInvM().GetData();
	for (int32 Index = 0; Index < Particles.GetRangeSize(); ++Index)
	{
		if (InvM[Index] != (FSolverReal)0.f)
		{
			Velocity[Index] += Acceleration[Index] * Dt;
		}
	}
}

static void DampLocalVelocity(FSolverParticlesRange& Particles, const FPerParticleDampVelocity& DampVelocityRule)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSoftsEvolution_DampLocalVelocity);
	// TODO: ISPC and/or ParallelFor
	FSolverVec3* const Velocity = Particles.GetV().GetData();
	const FSolverVec3* const X = Particles.XArray().GetData();
	const FSolverReal* const InvM = Particles.GetInvM().GetData();
	for (int32 Index = 0; Index < Particles.GetRangeSize(); ++Index)
	{
		if (InvM[Index] != (FSolverReal)0.f)
		{
			DampVelocityRule.Apply(X[Index], Velocity[Index]);
		}
	}
}

static void CalculateGlobalDamping(const FSolverReal DampingScale, const FSolverReal SolverFrequency, const FSolverReal Dt, FSolverReal& OutDampingPowDt, FSolverReal& OutDampingIntegrated)
{
	const FSolverReal Damping = FMath::Clamp(DampingScale, (FSolverReal)0., (FSolverReal)1.);
	if (Damping > (FSolverReal)1. - (FSolverReal)UE_KINDA_SMALL_NUMBER)
	{
		OutDampingIntegrated = OutDampingPowDt = (FSolverReal)0.;
	}
	else if (Damping > (FSolverReal)UE_SMALL_NUMBER)
	{
		const FSolverReal LogValueByFrequency = FMath::Loge((FSolverReal)1. - Damping) * SolverFrequency;

		OutDampingPowDt = FMath::Exp(LogValueByFrequency * Dt);  // DampingPowDt = FMath::Pow(OneMinusDamping, Dt * SolverFrequency);
		OutDampingIntegrated = (OutDampingPowDt - (FSolverReal)1.) / LogValueByFrequency;
	}
	else
	{
		OutDampingPowDt = (FSolverReal)1.;
		OutDampingIntegrated = Dt;
	}
}

static void EulerStepPositionWithGlobalDamping(FSolverParticlesRange& Particles, const FSolverReal DampingScale, const FSolverReal SolverFrequency, const FSolverReal Dt)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSoftsEvolution_EulerStepPositionWithGlobalDamping);
	FSolverReal DampingPowDt;
	FSolverReal DampingIntegrated;
	CalculateGlobalDamping(DampingScale, SolverFrequency, Dt, DampingPowDt, DampingIntegrated);
	const FSolverVec3* const X = Particles.XArray().GetData();
	FSolverVec3* const Velocity = Particles.GetV().GetData();
	FPAndInvM* const PAndInvM = Particles.GetPAndInvM().GetData();
	for (int32 Index = 0; Index < Particles.GetRangeSize(); ++Index)
	{
		if (PAndInvM[Index].InvM != (FSolverReal)0.)
		{
			PAndInvM[Index].P = X[Index] + Velocity[Index] * DampingIntegrated;
			Velocity[Index] *= DampingPowDt;
		}
	}
}

static void EulerStepPositionWithGlobalDampingArray(FSolverParticlesRange& Particles, const TArrayCollectionArray<FSolverReal>& ParticleDampings, const FSolverReal SolverFrequency, const FSolverReal Dt)
{
	const FSolverVec3* const X = Particles.XArray().GetData();
	FSolverVec3* const Velocity = Particles.GetV().GetData();
	FPAndInvM* const PAndInvM = Particles.GetPAndInvM().GetData();
	const FSolverReal* const DampingScale = Particles.GetConstArrayView(ParticleDampings).GetData();
	for (int32 Index = 0; Index < Particles.GetRangeSize(); ++Index)
	{
		if (PAndInvM[Index].InvM != (FSolverReal)0.)
		{
			FSolverReal DampingPowDt;
			FSolverReal DampingIntegrated;
			CalculateGlobalDamping(DampingScale[Index], SolverFrequency, Dt, DampingPowDt, DampingIntegrated);
			PAndInvM[Index].P = X[Index] + Velocity[Index] * DampingIntegrated;
			Velocity[Index] *= DampingPowDt;
		}
	}
}

static void QuasistaticUpdate(FSolverParticlesRange& Particles)
{
	const FSolverVec3* const X = Particles.XArray().GetData();
	FPAndInvM* const PAndInvM = Particles.GetPAndInvM().GetData();
	for (int32 Index = 0; Index < Particles.GetRangeSize(); ++Index)
	{
		if (PAndInvM[Index].InvM != (FSolverReal)0.)
		{
			PAndInvM[Index].P = X[Index];
		}
	}
}

template<bool bUpdateX>
static void PostStepUpdate(FSolverParticlesRange& Particles, const FSolverReal Dt)
{
	const FPAndInvM* const PAndInvM = Particles.GetPAndInvM().GetData();
	FSolverVec3* const X = Particles.XArray().GetData();
	FSolverVec3* const V = Particles.GetV().GetData();
	FSolverVec3* const VPrev = Particles.GetVPrev().GetData();
	for (int32 Index = 0; Index < Particles.GetRangeSize(); ++Index)
	{
		VPrev[Index] = V[Index];
		V[Index] = (PAndInvM[Index].P - X[Index]) / Dt;
		if constexpr (bUpdateX)
		{
			X[Index] = PAndInvM[Index].P;
		}
	}
}

static void CopyVtoVPrev(FSolverParticlesRange& Particles)
{
	const FSolverVec3* const V = Particles.GetV().GetData();
	FSolverVec3* const VPrev = Particles.GetVPrev().GetData();
	for (int32 Index = 0; Index < Particles.GetRangeSize(); ++Index)
	{
		VPrev[Index] = V[Index];
	}
}

static void CopyPToX(FSolverParticlesRange& Particles)
{
	const FPAndInvM* const PAndInvM = Particles.GetPAndInvM().GetData();
	FSolverVec3* const X = Particles.XArray().GetData();
	for (int32 Index = 0; Index < Particles.GetRangeSize(); ++Index)
	{
		X[Index] = PAndInvM[Index].P;
	}
}
}

FEvolution::FEvolution(const FCollectionPropertyConstFacade& Properties)
	: Time((FSolverReal)0.f)
{
	SetSolverProperties(Properties);
	Particles.AddArray(&ParticleDampings);
}

void FEvolution::Reset(bool bResetGlobalRules)
{
	CollisionParticles.Resize(0);
	Particles.Resize(0);

	SoftBodies.Reset();
	CollisionRanges.Reset();

	Groups.Reset();

	if (bResetGlobalRules)
	{
		KinematicUpdate.Reset();
		CollisionKinematicUpdate.Reset();
	}
}

TArray<uint32> FEvolution::GetActiveGroupsArray() const
{
	TArray<uint32> ActiveGroups;
	ActiveGroups.Reserve(Groups.Size());
	for (uint32 GroupId = 0; GroupId < Groups.Size(); ++GroupId)
	{
		if (!Groups.ActiveSoftBodies.IsEmpty())
		{
			ActiveGroups.Add(GroupId);
		}
	}
	return ActiveGroups;
}

int32 FEvolution::NumActiveParticles() const
{
	int32 Result = 0;
	for (const uint32 GroupId : GetActiveGroupsArray())
	{
		for (const int32 SoftBodyId : GetGroupActiveSoftBodies(GroupId))
		{
			Result += GetSoftBodyParticleNum(SoftBodyId);
		}
	}
	return Result;
}

int32 FEvolution::AddSoftBody(uint32 GroupId, int32 NumParticles, bool bEnable)
{
	if (GroupId >= Groups.Size())
	{
		Groups.AddGroupsToSize(GroupId + 1);
	}

	int32 SoftBodyId = INDEX_NONE;
	TArray<int32>* const FreeList = SoftBodyFreeList.Find(NumParticles);
	if (FreeList && FreeList->Num())
	{
		SoftBodyId = FreeList->Pop();
		check(SoftBodies.Status[SoftBodyId] == FSoftBodies::EStatus::Free);
	}
	else
	{
		SoftBodyId = SoftBodies.AddSoftBody();
		SoftBodies.ParticleRanges[SoftBodyId] = FSolverParticlesRange::AddParticleRange(Particles, NumParticles, SoftBodyId);
	}

	Groups.SoftBodies[GroupId].Emplace(SoftBodyId);

	// Set properties
	SoftBodies.Status[SoftBodyId] = FSoftBodies::EStatus::Inactive;
	SoftBodies.GroupId[SoftBodyId] = GroupId;
	SoftBodies.GlobalDampings[SoftBodyId] = Private::EvolutionSoftBodyDefault::GlobalDamping;
	SoftBodies.LocalDamping[SoftBodyId] = FPerParticleDampVelocity(Private::EvolutionSoftBodyDefault::LocalDamping);
	SoftBodies.UsePerParticleDamping[SoftBodyId] = Private::EvolutionSoftBodyDefault::bUsePerParticleDamping;
	SoftBodies.PreSubstepParallelInits[SoftBodyId].Reset();
	SoftBodies.PBDExternalForceRules[SoftBodyId].Reset();
	SoftBodies.PostInitialGuessParallelInits[SoftBodyId].Reset();
	SoftBodies.PreSubstepConstraintRules[SoftBodyId].Reset();
	SoftBodies.PerIterationPBDConstraintRules[SoftBodyId].Reset();
	SoftBodies.PerIterationCollisionPBDConstraintRules[SoftBodyId].Reset();
	SoftBodies.PerIterationPostCollisionsPBDConstraintRules[SoftBodyId].Reset();
	SoftBodies.UpdateLinearSystemRules[SoftBodyId].Reset();
	SoftBodies.UpdateLinearSystemCollisionsRules[SoftBodyId].Reset();
	SoftBodies.PostSubstepConstraintRules[SoftBodyId].Reset();
	ActivateSoftBody(SoftBodyId, bEnable);
	return SoftBodyId;
}

void FEvolution::RemoveSoftBody(int32 SoftBodyId)
{
	check(SoftBodies.Status[SoftBodyId] == FSoftBodies::EStatus::Active || SoftBodies.Status[SoftBodyId] == FSoftBodies::EStatus::Inactive);

	// Remove from active group if it was active.
	Groups.ActiveSoftBodies[SoftBodies.GroupId[SoftBodyId]].Remove(SoftBodyId);

	// Mark free
	SoftBodies.Status[SoftBodyId] = FSoftBodies::EStatus::Free;

	// Add to free list
	const int32 NumParticles = SoftBodies.ParticleRanges[SoftBodyId].GetRangeSize();
	TArray<int32>& FreeList = SoftBodyFreeList.FindOrAdd(NumParticles);
	FreeList.Add(SoftBodyId);
}


void FEvolution::SetSoftBodyProperties(int32 SoftBodyId, const FCollectionPropertyConstFacade& PropertyCollection,
	const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
	const FSolverVec3& ReferenceSpaceLocation,
	const FSolverVec3& ReferenceSpaceVelocity,
	const FSolverVec3& ReferenceSpaceAngularVelocity)
{
	SoftBodies.LocalDamping[SoftBodyId].SetProperties(PropertyCollection, ReferenceSpaceLocation, ReferenceSpaceVelocity, ReferenceSpaceAngularVelocity);

	const FSolverVec2 GlobalDamping = GetWeightedFloatDampingCoefficient(PropertyCollection, Private::EvolutionSoftBodyDefault::GlobalDamping);
	TConstArrayView<FRealSingle> WeightMap = WeightMaps.FindRef(GetDampingCoefficientString(PropertyCollection, DampingCoefficientName.ToString()));
	SoftBodies.UsePerParticleDamping[SoftBodyId] = WeightMap.Num() == GetSoftBodyParticleNum(SoftBodyId);
	if (SoftBodies.UsePerParticleDamping[SoftBodyId])
	{
		const FSolverParticlesRange& SoftBodyParticles = GetSoftBodyParticles(SoftBodyId);
		TArrayView<FSolverReal> PerParticleDamping = SoftBodyParticles.GetArrayView(ParticleDampings);
		check(PerParticleDamping.Num() == WeightMap.Num());
		const FPBDFlatWeightMapView GlobalDampingMap(GlobalDamping, WeightMap, PerParticleDamping.Num());
		for (int32 Index = 0; Index < PerParticleDamping.Num(); ++Index)
		{
			PerParticleDamping[Index] = GlobalDampingMap.GetValue(Index);
		}
	}
	else
	{
		SoftBodies.GlobalDampings[SoftBodyId] = GlobalDamping[0];
	}
}

void FEvolution::ActivateSoftBody(int32 SoftBodyId, bool bActivate)
{
	check(SoftBodies.Status[SoftBodyId] != FSoftBodies::EStatus::Free);
	SoftBodies.Status[SoftBodyId] = bActivate ? FSoftBodies::EStatus::Active : FSoftBodies::EStatus::Inactive;
	if (bActivate)
	{
		Groups.ActiveSoftBodies[SoftBodies.GroupId[SoftBodyId]].Add(SoftBodyId);
	}
	else
	{
		Groups.ActiveSoftBodies[SoftBodies.GroupId[SoftBodyId]].Remove(SoftBodyId);
	}
}

int32 FEvolution::AddCollisionParticleRange(uint32 GroupId, int32 NumParticles, bool bEnable)
{
	if (GroupId >= Groups.Size())
	{
		Groups.AddGroupsToSize(GroupId + 1);
	}

	int32 CollisionRangeId = INDEX_NONE;
	TArray<int32>* const FreeList = CollisionRangeFreeList.Find(NumParticles);
	if (FreeList && FreeList->Num())
	{
		CollisionRangeId = FreeList->Pop();
		check(CollisionRanges.Status[CollisionRangeId] == FCollisionBodyRanges::EStatus::Free);
	}
	else
	{
		CollisionRangeId = CollisionRanges.AddRange();
		CollisionRanges.ParticleRanges[CollisionRangeId] = FSolverCollisionParticlesRange::AddParticleRange(CollisionParticles, NumParticles, CollisionRangeId);
	}

	// Set properties
	CollisionRanges.Status[CollisionRangeId] = FCollisionBodyRanges::EStatus::Inactive;
	CollisionRanges.GroupId[CollisionRangeId] = GroupId;
	ActivateCollisionParticleRange(CollisionRangeId, bEnable);
	return CollisionRangeId;
}

void FEvolution::RemoveCollisionParticleRange(int32 CollisionRangeId)
{
	check(CollisionRanges.Status[CollisionRangeId] == FCollisionBodyRanges::EStatus::Active || CollisionRanges.Status[CollisionRangeId] == FCollisionBodyRanges::EStatus::Inactive);

	// Remove from active group if it was active.
	Groups.ActiveCollisionParticleRanges[CollisionRanges.GroupId[CollisionRangeId]].Remove(CollisionRangeId);

	// Mark free
	CollisionRanges.Status[CollisionRangeId] = FCollisionBodyRanges::EStatus::Free;

	// Add to free list
	const int32 NumParticles = CollisionRanges.ParticleRanges[CollisionRangeId].GetRangeSize();
	TArray<int32>& FreeList = CollisionRangeFreeList.FindOrAdd(NumParticles);
	FreeList.Add(CollisionRangeId);
}

void FEvolution::ActivateCollisionParticleRange(int32 CollisionRangeId, bool bEnable)
{
	check(CollisionRanges.Status[CollisionRangeId] != FCollisionBodyRanges::EStatus::Free);

	CollisionRanges.Status[CollisionRangeId] = bEnable ? FCollisionBodyRanges::EStatus::Active : FCollisionBodyRanges::EStatus::Inactive;
	if (bEnable)
	{
		Groups.ActiveCollisionParticleRanges[CollisionRanges.GroupId[CollisionRangeId]].Add(CollisionRangeId);
	}
	else
	{
		Groups.ActiveCollisionParticleRanges[CollisionRanges.GroupId[CollisionRangeId]].Remove(CollisionRangeId);
	}

}

TArray<FSolverCollisionParticlesRange> FEvolution::GetActiveCollisionParticles(uint32 GroupId) const
{
	TArray<FSolverCollisionParticlesRange> Ranges;
	Ranges.Reserve(Groups.ActiveCollisionParticleRanges[GroupId].Num());
	for (const int32 RangeId : Groups.ActiveCollisionParticleRanges[GroupId])
	{
		check(CollisionRanges.Status[RangeId] == FCollisionBodyRanges::EStatus::Active);
		check(CollisionRanges.GroupId[RangeId] == GroupId);
		Ranges.Emplace(CollisionRanges.ParticleRanges[RangeId]);
	}
	return Ranges;
}

void FEvolution::SetSolverProperties(const FCollectionPropertyConstFacade& Properties)
{
	bEnableForceBasedSolver = GetEnableForceBasedSolver(Properties, Private::EvolutionSolverDefault::bEnableForceBasedSolver);
	MaxNumIterations = FMath::Max(GetMaxNumIterations(Properties, Private::EvolutionSolverDefault::MaxNumIterations), Private::EvolutionSolverDefault::MinNumIterations);
	NumIterations = GetNumIterations(Properties, Private::EvolutionSolverDefault::NumIterations);
	bDoQuasistatics = GetDoQuasistatics(Properties, Private::EvolutionSolverDefault::bDoQuasistatics);
	SolverFrequency = GetSolverFrequency(Properties, Private::EvolutionSolverDefault::SolverFrequency);
	NumNewtonIterations = GetNumNewtonIterations(Properties, Private::EvolutionSolverDefault::NumNewtonIterations);
	LinearSystemParameters.SetProperties(Properties, NumIterations > 0);
}

void FEvolution::AdvanceOneTimeStep(const FSolverReal Dt, const FSolverReal TimeDependentIterationMultiplier)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSoftsEvolution_AdvanceOneTimeStep);

	// Advance time
	Time += Dt;

	// Keep NumIterations == 0 as zero, but otherwise, clamp to at least 1.
	const int32 TimeDependentNumIterations = (bDisableTimeDependentNumIterations || NumIterations == 0) ? NumIterations :
		FMath::Clamp(FMath::RoundToInt32(SolverFrequency * Dt * TimeDependentIterationMultiplier * (Softs::FSolverReal)NumIterations),
			Private::EvolutionSolverDefault::MinNumIterations, MaxNumIterations);
	NumUsedIterations = TimeDependentNumIterations;

	const TArray<uint32> ActiveGroupsArray = GetActiveGroupsArray();

	PhysicsParallelFor(ActiveGroupsArray.Num(),
		[this, &ActiveGroupsArray, Dt, TimeDependentNumIterations](int32 ActiveGroupIndex)
	{
		AdvanceOneTimeStepInternal(Dt, TimeDependentNumIterations, ActiveGroupsArray[ActiveGroupIndex]);
	});
}

void FEvolution::AdvanceOneTimeStepInternal(const FSolverReal Dt, const int32 TimeDependentNumIterations, uint32 GroupId)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSoftsEvolution_AdvanceOneTimeStepInternal);

	FEvolutionGroupContext& Context = Groups.SolverContexts[GroupId];
	Context.Reset();

	TArray<FSolverCollisionParticlesRange> ActiveCollisionRanges = GetActiveCollisionParticles(GroupId);
	{
		// Collision kinematic update
		TRACE_CPUPROFILER_EVENT_SCOPE(FSoftsEvolution_AdvanceOneTimeStepInternal_CollisionKinematicUpdate);
		check(CollisionKinematicUpdate);
		for (FSolverCollisionParticlesRange& ActiveRange : ActiveCollisionRanges)
		{
			CollisionKinematicUpdate(ActiveRange, Dt, Time);
		}
	}

	// Right now, typically groups only have one active softbody in them, so don't parallelize softbodies
	for (const int32 SoftBodyId : Groups.ActiveSoftBodies[GroupId])
	{
		check(SoftBodies.Status[SoftBodyId] == FSoftBodies::EStatus::Active);

		const bool bDoNewtonUpdate = bEnableForceBasedSolver && NumNewtonIterations > 0 && LinearSystemParameters.MaxNumCGIterations > 0 && !SoftBodies.UpdateLinearSystemRules[SoftBodyId].IsEmpty();

		// Do at least one PBD iteration if not doing a newton update.
		const int32 NumPBDIterations = bDoNewtonUpdate ? TimeDependentNumIterations : FMath::Max(1, TimeDependentNumIterations);

		const ESolverMode SolverMode =
			(NumPBDIterations > 0 ? ESolverMode::PBD : ESolverMode::None) |
			(bDoNewtonUpdate ? ESolverMode::ForceBased : ESolverMode::None);
		check(SolverMode != ESolverMode::None);

		Context.Init(SolverMode, Dt, NumPBDIterations, bDoNewtonUpdate ? NumNewtonIterations : 0);

		// PreSubstepParallelInits + LocalDamping
		FPerParticleDampVelocity& DampVelocityRule = SoftBodies.LocalDamping[SoftBodyId];
		const bool bDoLocalDampingPositionUpdate = DampVelocityRule.RequiresUpdatePositionBasedState();
		const bool bDoLocalDamping = DampVelocityRule.IsEnabled();
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FSoftsEvolution_AdvanceOneTimeStepInternal_PreSubstepParallelInits);

			TConstArrayView<ParallelInitFunc> PreSubstepParallelInits(SoftBodies.PreSubstepParallelInits[SoftBodyId]);
			const int32 NumPreSubstepInits = PreSubstepParallelInits.Num() + (bDoLocalDampingPositionUpdate ? 1 : 0);
			PhysicsParallelFor(NumPreSubstepInits,
				[this, &PreSubstepParallelInits, &DampVelocityRule, Dt, SoftBodyId, SolverMode](int32 FuncIdx)
			{
				if (FuncIdx == PreSubstepParallelInits.Num())
				{
					DampVelocityRule.UpdatePositionBasedState(SoftBodies.ParticleRanges[SoftBodyId]);
				}
				else
				{
					PreSubstepParallelInits[FuncIdx](SoftBodies.ParticleRanges[SoftBodyId], Dt, SolverMode);
				}
			}
			);
		}

		// TODO: hypothetically, kinematic, dynamic, and collision kinematic updates can run concurrently
		{
			// Kinematic update
			check(KinematicUpdate);
			TRACE_CPUPROFILER_EVENT_SCOPE(FSoftsEvolution_AdvanceOneTimeStepInternal_KinematicUpdates);
			KinematicUpdate(SoftBodies.ParticleRanges[SoftBodyId], Dt, Time);
		}

		// PBD initial guess
		if (NumPBDIterations > 0)
		{
			{
				// Dynamic initial guess (Euler step external forces and apply damping)
				TRACE_CPUPROFILER_EVENT_SCOPE(FSoftsEvolution_AdvanceOneTimeStepInternal_InitialGuess);
				if (bDoQuasistatics)
				{
					Private::QuasistaticUpdate(SoftBodies.ParticleRanges[SoftBodyId]);
				}
				else
				{
					// ExternalForceRules
					TConstArrayView<PBDConstraintRuleFunc> ExternalForceRules(SoftBodies.PBDExternalForceRules[SoftBodyId]);
					for (const PBDConstraintRuleFunc& ExternalForceRule : ExternalForceRules)
					{
						ExternalForceRule(SoftBodies.ParticleRanges[SoftBodyId], Dt);
					}

					// Euler Step Velocity
					Private::EulerStepVelocity(SoftBodies.ParticleRanges[SoftBodyId], Dt);

					if (bDoLocalDamping)
					{
						Private::DampLocalVelocity(SoftBodies.ParticleRanges[SoftBodyId], DampVelocityRule);
					}

					if (SoftBodies.UsePerParticleDamping[SoftBodyId])
					{
						Private::EulerStepPositionWithGlobalDampingArray(SoftBodies.ParticleRanges[SoftBodyId], ParticleDampings, SolverFrequency, Dt);
					}
					else
					{
						Private::EulerStepPositionWithGlobalDamping(SoftBodies.ParticleRanges[SoftBodyId], SoftBodies.GlobalDampings[SoftBodyId], SolverFrequency, Dt);
					}
				}
			}
		}

		{
			// Parallel Inits
			TRACE_CPUPROFILER_EVENT_SCOPE(FSoftsEvolution_AdvanceOneTimeStepInternal_PostInitialGuessParallelInits);
			TConstArrayView<ParallelInitFunc> ConstraintParallelInits(SoftBodies.PostInitialGuessParallelInits[SoftBodyId]);
			PhysicsParallelFor(ConstraintParallelInits.Num(),
				[this, &ConstraintParallelInits, Dt, SoftBodyId, SolverMode](int32 Index)
			{
				ConstraintParallelInits[Index](SoftBodies.ParticleRanges[SoftBodyId], Dt, SolverMode);
			});
		}
		{
			// PreSubstep rules
			TRACE_CPUPROFILER_EVENT_SCOPE(FSoftsEvolution_AdvanceOneTimeStepInternal_PreSubstepConstraintRules);
			TConstArrayView<ConstraintRuleFunc> PreSubstepConstraintRules(SoftBodies.PreSubstepConstraintRules[SoftBodyId]);
			for (const ConstraintRuleFunc& ConstraintRule : PreSubstepConstraintRules)
			{
				ConstraintRule(SoftBodies.ParticleRanges[SoftBodyId], Dt, SolverMode);
			}
		}
		if (NumPBDIterations > 0)
		{
			{
				// Iteration loop
				TRACE_CPUPROFILER_EVENT_SCOPE(FSoftsEvolution_AdvanceOneTimeStepInternal_IterationLoop);
				for (Context.CurrentPBDIteration = 0; Context.CurrentPBDIteration < NumPBDIterations; ++Context.CurrentPBDIteration)
				{

					TConstArrayView<PBDConstraintRuleFunc> PerIterationConstraintRules(SoftBodies.PerIterationPBDConstraintRules[SoftBodyId]);
					for (const PBDConstraintRuleFunc& ConstraintRule : PerIterationConstraintRules)
					{
						ConstraintRule(SoftBodies.ParticleRanges[SoftBodyId], Dt);
					}

					TConstArrayView<PBDCollisionConstraintRuleFunc> PerIterationCollisionConstraintRules(SoftBodies.PerIterationCollisionPBDConstraintRules[SoftBodyId]);
					for (const PBDCollisionConstraintRuleFunc& ConstraintRule : PerIterationCollisionConstraintRules)
					{
						ConstraintRule(SoftBodies.ParticleRanges[SoftBodyId], Dt, ActiveCollisionRanges);
					}

					TConstArrayView<PBDConstraintRuleFunc> PerIterationPostCollisionsConstraintRules(SoftBodies.PerIterationPostCollisionsPBDConstraintRules[SoftBodyId]);
					for (const PBDConstraintRuleFunc& ConstraintRule : PerIterationPostCollisionsConstraintRules)
					{
						ConstraintRule(SoftBodies.ParticleRanges[SoftBodyId], Dt);
					}
				}
			}
			{
				// Post Step Update
				TRACE_CPUPROFILER_EVENT_SCOPE(FSoftsEvolution_AdvanceOneTimeStepInternal_PostStepUpdate);
				if (bDoNewtonUpdate)
				{
					Private::PostStepUpdate<false>(SoftBodies.ParticleRanges[SoftBodyId], Dt);
				}
				else
				{
					Private::PostStepUpdate<true>(SoftBodies.ParticleRanges[SoftBodyId], Dt);
				}
			}
		}
		else
		{
			Private::CopyVtoVPrev(SoftBodies.ParticleRanges[SoftBodyId]);
		}
		
		// Linear system update... currently we don't have any cross-softbody constraints. 
		// If we did, the linear system would need to live at the group level.
		if (bDoNewtonUpdate)
		{
			TConstArrayView<UpdateLinearSystemFunc> LinearSystemRules(SoftBodies.UpdateLinearSystemRules[SoftBodyId]);
			check(LinearSystemRules.Num());
			TConstArrayView<UpdateLinearSystemCollisionsFunc> LinearSystemCollisionRules(SoftBodies.UpdateLinearSystemCollisionsRules[SoftBodyId]);
			check(NumNewtonIterations > 0);
			for (Context.CurrentNewtonIteration = 0; Context.CurrentNewtonIteration < NumNewtonIterations; ++Context.CurrentNewtonIteration)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FSoftsEvolution_AdvanceOneTimeStepInternal_NewtonIteration);
				SoftBodies.LinearSystems[SoftBodyId].Init(SoftBodies.ParticleRanges[SoftBodyId], Dt, Context.CurrentNewtonIteration == 0, LinearSystemParameters);

				for (const UpdateLinearSystemFunc& LinearSystemRule : LinearSystemRules)
				{
					LinearSystemRule(SoftBodies.ParticleRanges[SoftBodyId], Dt, SoftBodies.LinearSystems[SoftBodyId]);
				}

				for (const UpdateLinearSystemCollisionsFunc& LinearSystemCollisionRule : LinearSystemCollisionRules)
				{
					LinearSystemCollisionRule(SoftBodies.ParticleRanges[SoftBodyId], Dt, ActiveCollisionRanges, SoftBodies.LinearSystems[SoftBodyId]);
				}

				if (!SoftBodies.LinearSystems[SoftBodyId].Solve(SoftBodies.ParticleRanges[SoftBodyId], Dt))
				{
					// Linear system update failed... just bail for now
					break;
				}
			}
			Private::CopyPToX(SoftBodies.ParticleRanges[SoftBodyId]);
		}

		{
			// Post Substep Rules
			TRACE_CPUPROFILER_EVENT_SCOPE(FSoftsEvolution_AdvanceOneTimeStepInternal_PostSubstep);
			TConstArrayView<ConstraintRuleFunc> PostSubstepConstraintRules(SoftBodies.PostSubstepConstraintRules[SoftBodyId]);
			for (const ConstraintRuleFunc& ConstraintRule : PostSubstepConstraintRules)
			{
				ConstraintRule(SoftBodies.ParticleRanges[SoftBodyId], Dt, SolverMode);
			}
		}
	}
}
}