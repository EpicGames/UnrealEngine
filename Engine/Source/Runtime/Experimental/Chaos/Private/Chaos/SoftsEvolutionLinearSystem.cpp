// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/SoftsEvolutionLinearSystem.h"
#include "Chaos/SoftsSolverParticlesRange.h"

namespace Chaos::Softs
{

FEvolutionLinearSystem::FEvolutionLinearSystem(const FSolverParticlesRange& Particles)
{
	CalculateCompactIndices(Particles);
	RHS.Init(FSolverVec3::ZeroVector, NumCompactIndices);
}

void FEvolutionLinearSystem::CalculateCompactIndices(const FSolverParticlesRange& Particles)
{
	NumCompactIndices = 0;
	CompactifiedIndices.SetNumUninitialized(Particles.GetRangeSize());
	for (int32 Index = 0; Index < Particles.GetRangeSize(); ++Index)
	{
		if (Particles.InvM(Index) == (FSolverReal)0.)
		{
			CompactifiedIndices[Index] = INDEX_NONE;
		}
		else
		{
			CompactifiedIndices[Index] = NumCompactIndices++;
		}
	}
}

void FEvolutionLinearSystem::Init(const FSolverParticlesRange& Particles, const FSolverReal Dt, bool bInFirstNewtonIteration, const FEvolutionLinearSystemSolverParameters& Params)
{
	Parameters = &Params;
	if (CompactifiedIndices.Num() != Particles.Size())
	{
		CalculateCompactIndices(Particles);
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FEvolutionLinearSystem_Init);
	// Note on variable naming convention
	// X = Last time position
	// P = Current time position guess
	// VPrev = Last time velocity
	// V = Current time velocity guess
	//
	// When solving for DeltaV,
	// Solve M*(VNext - VPrev)/Dt = F, (XNext - X)/Dt = VNext
	// Linearize about current guess of P and V, so 
	// VNext = V + DeltaV, XNext = P + DeltaX
	//  
	// M*(V + DeltaV - VPrev) / Dt = F + DfDx * DeltaX + DfDv * DeltaV
	// (P + DeltaX - X)/Dt = V + DeltaV
	// Substitute DeltaX = (V + DeltaV) * Dt - (P - X), multiply by Dt.
	// 
	// (M - DfDx * Dt * Dt - DfDv * Dt) * DeltaV = F * Dt + M * (VPrev - V) + DfDx * (X - P + V * Dt) * Dt

	// When using XPBD Initial Guess, or we're not the first Newton Iteration, P and V have been updated such that
	// (1) P = X + V * Dt
	// When first Newton Iteration, no XPBD,
	// (2) P = X, V = VPrev
	// Thus, in case (1), the RHS is
	// F * Dt + M * (VPrev - V)
	// In case (2), the RHS is
	// F * Dt + DfDx * V * Dt * Dt

	// When doing quasistatics and solve for DeltaX
	// Solving F = 0
	// Linearize about XNext = P + DeltaX
	//
	// -DfDx * DeltaX = F
	// 
	bDfDxTimesVTerm = !Parameters->bDoQuasistatics && !Parameters->bXPBDInitialGuess && bInFirstNewtonIteration;

	// Initialize RHS
	if (bDfDxTimesVTerm || Parameters->bDoQuasistatics)
	{
		RHS.Init(FSolverVec3::ZeroVector, NumCompactIndices);
	}
	else
	{
		check(CompactifiedIndices.Num() == Particles.GetRangeSize());

		RHS.SetNumUninitialized(NumCompactIndices);
		const FSolverReal* const M = Particles.GetM().GetData();
		const FSolverVec3* const V = Particles.GetV().GetData();
		const FSolverVec3* const VPrev = Particles.GetVPrev().GetData();
		for (int32 Index = 0; Index < Particles.GetRangeSize(); ++Index)
		{
			if (CompactifiedIndices[Index] != INDEX_NONE)
			{
				RHS[CompactifiedIndices[Index]] = M[Index] * (VPrev[Index] - V[Index]);
			}
		}
	}

	// Initialize matrix as M when not doing quasistatics
	Matrix.Reset(NumCompactIndices);
	if (!Parameters->bDoQuasistatics)
	{
		Matrix.ReserveForParallelAdd(NumCompactIndices, 0);
		const FSolverReal* const M = Particles.GetM().GetData();
		for (int32 Index = 0; Index < Particles.GetRangeSize(); ++Index)
		{
			const int32 CompactIndex = CompactifiedIndices[Index];
			if (CompactIndex != INDEX_NONE)
			{
				Matrix.AddMatrixEntry(CompactIndex, CompactIndex, FSolverMatrix33(M[Index], M[Index], M[Index]));
			}
		}
	}
}

void FEvolutionLinearSystem::AddForce(const FSolverParticlesRange& Particles, const FSolverVec3& Force, int32 ParticleIndex, const FSolverReal Dt)
{
	const int32 CompactIndex = CompactifiedIndices[ParticleIndex];
	
	const FSolverReal DtScale = Parameters->bDoQuasistatics ? 1.f : Dt;
	// RHS force contribution = F * Dt;
	if (CompactIndex != INDEX_NONE)
	{
		RHS[CompactIndex] += Force * DtScale;
	}
}

void FEvolutionLinearSystem::AddSymmetricForceDerivative(const FSolverParticlesRange& Particles, const FSolverMatrix33* const Df1Dx2, const FSolverMatrix33* const Df1Dv2, int32 ParticleIndex1, int32 ParticleIndex2, const FSolverReal Dt)
{
	check(Parameters);

	const int32 CompactIndex1 = CompactifiedIndices[ParticleIndex1];
	const int32 CompactIndex2 = CompactifiedIndices[ParticleIndex2];
	if (Parameters->bDoQuasistatics)
	{
		if (CompactIndex1 != INDEX_NONE && CompactIndex2 != INDEX_NONE && Df1Dx2)
		{
			// -DfDx
			Matrix.AddMatrixEntry(CompactIndex1, CompactIndex2, -(*Df1Dx2));
		}
	}
	else
	{
		if(CompactIndex1 != INDEX_NONE && CompactIndex2 != INDEX_NONE)
		{
			// - DfDx * Dt * Dt - DfDv * Dt
			FSolverMatrix33 MatrixToAdd;
			if (Df1Dx2)
			{
				MatrixToAdd = (*Df1Dx2) * (-Dt * Dt);
				if (Df1Dv2)
				{
					MatrixToAdd += (*Df1Dv2) * (-Dt);
				}
			}
			else if (Df1Dv2)
			{
				MatrixToAdd = (*Df1Dv2) * (-Dt);
			}
			else
			{
				return;
			}
			Matrix.AddMatrixEntry(CompactIndex1, CompactIndex2, MatrixToAdd);
		} // else Dirichlet boundaries with DeltaV (or DeltaX) = 0.

		if (bDfDxTimesVTerm && Df1Dx2)
		{
			// DfDx * V * Dt * Dt
			if (CompactIndex1 != INDEX_NONE)
			{
				RHS[CompactIndex1] += Df1Dx2->TransformVector(Particles.V(ParticleIndex1)) * (Dt * Dt);
			}
			if (CompactIndex2 != INDEX_NONE && CompactIndex1 != CompactIndex2)
			{
				RHS[CompactIndex2] += (*Df1Dx2) * Particles.V(ParticleIndex2) * (Dt * Dt);
			}
		}
	}
}

bool FEvolutionLinearSystem::Solve(FSolverParticlesRange& Particles, const FSolverReal Dt)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FEvolutionLinearSystem_Solve);
	check(Parameters);
	Matrix.FinalizeSystem();

	TArray<FSolverVec3> DeltaCompact;
	DeltaCompact.Init(FSolverVec3::ZeroVector, NumCompactIndices);

	const bool bSuccess = Matrix.Solve(TConstArrayView<FSolverVec3>(RHS), TArrayView<FSolverVec3>(DeltaCompact), Parameters->MaxNumCGIterations, Parameters->CGResidualTolerance, Parameters->bCheckCGResidual, &LastSolveIterations, &LastSolveError);
	if (bSuccess)
	{
		if (Parameters->bDoQuasistatics)
		{
			// Solved for DeltaX
			FPAndInvM* const PAndInvM = Particles.GetPAndInvM().GetData();
			for (int32 ParticleIndex = 0; ParticleIndex < Particles.GetRangeSize(); ++ParticleIndex)
			{
				if (CompactifiedIndices[ParticleIndex] != INDEX_NONE)
				{
					PAndInvM[ParticleIndex].P += DeltaCompact[CompactifiedIndices[ParticleIndex]];
				}
			}
		}
		else
		{
			// Solved for DeltaV
			if (bDfDxTimesVTerm)
			{
				const FSolverVec3* const X = Particles.XArray().GetData();
				FPAndInvM* const PAndInvM = Particles.GetPAndInvM().GetData();
				FSolverVec3* const V = Particles.GetV().GetData();
				for (int32 ParticleIndex = 0; ParticleIndex < Particles.GetRangeSize(); ++ParticleIndex)
				{
					if (CompactifiedIndices[ParticleIndex] != INDEX_NONE)
					{
						V[ParticleIndex] += DeltaCompact[CompactifiedIndices[ParticleIndex]];
						PAndInvM[ParticleIndex].P = X[ParticleIndex] + V[ParticleIndex] * Dt;
					}
				}
			}
			else
			{
				FPAndInvM* const PAndInvM = Particles.GetPAndInvM().GetData();
				FSolverVec3* const V = Particles.GetV().GetData();
				for (int32 ParticleIndex = 0; ParticleIndex < Particles.GetRangeSize(); ++ParticleIndex)
				{
					if (CompactifiedIndices[ParticleIndex] != INDEX_NONE)
					{
						V[ParticleIndex] += DeltaCompact[CompactifiedIndices[ParticleIndex]];
						PAndInvM[ParticleIndex].P += DeltaCompact[CompactifiedIndices[ParticleIndex]] * Dt;
					}
				}
			}
		}
		return true;
	}
	return false;
}

}  // End namespace Chaos::Softs
