// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/SoftsMultiResConstraints.h"
#include "Chaos/Framework/Parallel.h"
#include "Chaos/PBDSoftsSolverParticles.h"
#include "Chaos/SoftsSpring.h"
#include "ChaosStats.h"
#include "ChaosLog.h"
#include "HAL/IConsoleManager.h"

#include "XPBDInternal.h"

#if INTEL_ISPC
#include "SoftsMultiResConstraints.ispc.generated.h"
#endif

#if INTEL_ISPC && !UE_BUILD_SHIPPING
static_assert(sizeof(ispc::FVector4f) == sizeof(Chaos::Softs::FPAndInvM), "sizeof(ispc::FVector4f) != sizeof(Chaos::Softs::FPAndInvM)");
static_assert(sizeof(ispc::FIntVector) == sizeof(Chaos::TVec3<int32>), "sizeof(ispc::FIntVector) != sizeof(Chaos::TVec3<int32>");
static_assert(sizeof(ispc::FVector4f) == sizeof(Chaos::TVec4<Chaos::Softs::FSolverReal>), "sizeof(ispc::FVector4f) == sizeof(Chaos::TVec4<Chaos::Softs::FSolverReal>)");
static_assert(sizeof(ispc::FIntVector) == sizeof(Chaos::TVec3<int32>), "sizeof(ispc::FIntVector) != sizeof(Chaos::TVec3<int32>");

bool bChaos_MultiRes_ISPC_Enabled = CHAOS_MULTIRES_ISPC_ENABLED_DEFAULT;
FAutoConsoleVariableRef CVarChaosMultiResISPCEnabled(TEXT("p.Chaos.MultiRes.ISPC"), bChaos_MultiRes_ISPC_Enabled, TEXT("Whether to use ISPC optimizations in MultiRes constraints"));
bool bChaos_MultiRes_SparseWeightMap_Enabled = false;
FAutoConsoleVariableRef CVarChaosMultiResSparseWeightMapEnabled(TEXT("p.Chaos.MultiRes.SparseWeightMap"), bChaos_MultiRes_SparseWeightMap_Enabled, TEXT("Exploit the sparse weight map structure and skip the particles with 0 stiffness at the beginning and at the end"));
#endif

namespace Chaos::Softs {

static bool bMultiResConstraintApplyTargetNormalOffset = true;
static FAutoConsoleVariableRef CVarMultiResConstraintApplyTargetNormalOffset(TEXT("p.Chaos.MultiRes.ApplyTargetNormalOffset"), bMultiResConstraintApplyTargetNormalOffset, TEXT("Apply normal offset to targets."));

void FMultiResConstraints::SetProperties(const FCollectionPropertyConstFacade& PropertyCollection,
	const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps)
{
	if (IsMultiResUseXPBDMutable(PropertyCollection))
	{
		bUseXPBD = GetMultiResUseXPBD(PropertyCollection);
	}
	if (IsMultiResStiffnessMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedValue(GetWeightedFloatMultiResStiffness(PropertyCollection));
		if (IsMultiResStiffnessStringDirty(PropertyCollection))
		{
			const FString& WeightMapName = GetMultiResStiffnessString(PropertyCollection);
			Stiffness = FPBDStiffness(WeightedValue,
				WeightMaps.FindRef(WeightMapName),
				NumParticles,
				FPBDStiffness::DefaultTableSize,
				FPBDStiffness::DefaultParameterFitBase,
				bUseXPBD ? MaxStiffness : (FSolverReal)1.f);
		}
		else
		{
			Stiffness.SetWeightedValue(WeightedValue, bUseXPBD ? MaxStiffness : (FSolverReal)1.f);
		}
	}
	if (IsMultiResVelocityTargetStiffnessMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedValue(GetWeightedFloatMultiResVelocityTargetStiffness(PropertyCollection));
		if (IsMultiResVelocityTargetStiffnessStringDirty(PropertyCollection))
		{
			const FString& WeightMapName = GetMultiResVelocityTargetStiffnessString(PropertyCollection);
			VelocityTargetStiffness = FPBDStiffness(WeightedValue,
				WeightMaps.FindRef(WeightMapName),
				NumParticles,
				FPBDStiffness::DefaultTableSize,
				FPBDStiffness::DefaultParameterFitBase,
				bUseXPBD ? MaxStiffness : (FSolverReal)1.f);
		}
		else
		{
			VelocityTargetStiffness.SetWeightedValue(WeightedValue, bUseXPBD ? MaxStiffness : (FSolverReal)1.f);
		}
	}

	for (int32 Index = 0; Index < NumParticles; ++Index)
	{
		if (IsConstraintActive(Index))
		{
			NonZeroStiffnessMin = Index;
			break;
		}
	}
	for (int32 Index = NumParticles - 1; Index > INDEX_NONE; --Index)
	{
		if (IsConstraintActive(Index))
		{
			NonZeroStiffnessMax = Index;
			break;
		}
	}
	check(NonZeroStiffnessMax >= NonZeroStiffnessMin)
	if (NonZeroStiffnessMax != INDEX_NONE && NonZeroStiffnessMin != INDEX_NONE)
	{
		bStiffnessEntriesInitialized = true;
	}
	else
	{
		bStiffnessEntriesInitialized = false;
	}
}

void FMultiResConstraints::ApplyHelper(FSolverParticlesRange& Particles, const FSolverReal Dt, const int32 ConstraintIndex, const FSolverReal ExpStiffnessValue, const FSolverReal ExpVelocityTargetStiffnessValue) const
{
	if (bUseXPBD)
	{
		if (Particles.InvM(ConstraintIndex) != (FSolverReal)0.f)
		{
			const FSolverVec3& P1 = Particles.P(ConstraintIndex);
			const FSolverVec3& P2 = FineTargetPositions[ConstraintIndex];
			FSolverVec3 Direction = P1 - P2;
			const FSolverReal Offset = Direction.SafeNormalize();

			const FSolverReal AlphaInv = ExpStiffnessValue * Dt * Dt * (Particles.M(ConstraintIndex) + CoarseBarycentricMass[ConstraintIndex]) * 0.5f;
			const FSolverReal DLambda = (-AlphaInv * Offset - Lambdas[ConstraintIndex]) / (AlphaInv * Particles.InvM(ConstraintIndex) + (FSolverReal)1.);
			const FSolverVec3 Delta = DLambda * Direction;
			Lambdas[ConstraintIndex] += DLambda;
			Particles.P(ConstraintIndex) += Particles.InvM(ConstraintIndex) * Delta;
		}
	}
	else
	{
		if (Particles.InvM(ConstraintIndex) != (FSolverReal)0.f)
		{
			Particles.P(ConstraintIndex) -= ExpStiffnessValue * (Particles.P(ConstraintIndex) - FineTargetPositions[ConstraintIndex]);
			const FSolverVec3 FineDisplacement = Particles.P(ConstraintIndex) - Particles.X(ConstraintIndex);
			const FSolverVec3 TargetDisplacement = FineTargetVelocities[ConstraintIndex] * Dt;
			Particles.P(ConstraintIndex) -= ExpVelocityTargetStiffnessValue * (FineDisplacement - TargetDisplacement);
		}
	}
}

void FMultiResConstraints::Apply(FSolverParticlesRange& Particles, const FSolverReal Dt) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMultiResConstraints_Apply);
	if (FineTargetPositions.Num() != Particles.Size())
	{
		return;
	}
#if INTEL_ISPC
	if (!bUseXPBD && bChaos_MultiRes_ISPC_Enabled)
	{
		const bool bStiffnessHasWeightMap = Stiffness.HasWeightMap();
		const bool bVelocityStiffnessHasWeightMap = VelocityTargetStiffness.HasWeightMap();
		if (!bStiffnessHasWeightMap && !bVelocityStiffnessHasWeightMap)
		{
			ispc::ApplyMultiResConstraints(
				(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
				(const ispc::FVector3f*)Particles.XArray().GetData(),
				(const ispc::FVector3f*)FineTargetPositions.GetData(),
				(const ispc::FVector3f*)FineTargetVelocities.GetData(),
				Dt,
				(FSolverReal)Stiffness,
				(FSolverReal)VelocityTargetStiffness,
				Particles.Size()
				);
		}
		else
		{
#if !UE_BUILD_SHIPPING
			if (bChaos_MultiRes_SparseWeightMap_Enabled && bStiffnessEntriesInitialized)
			{
				check(NonZeroStiffnessMax != INDEX_NONE && NonZeroStiffnessMin != INDEX_NONE)
				ispc::ApplyMultiResConstraintsWithWeightMaps(
					(ispc::FVector4f*)&Particles.GetPAndInvM().GetData()[NonZeroStiffnessMin],
					(const ispc::FVector3f*)&Particles.XArray().GetData()[NonZeroStiffnessMin],
					(const ispc::FVector3f*)&FineTargetPositions.GetData()[NonZeroStiffnessMin],
					(const ispc::FVector3f*)&FineTargetVelocities.GetData()[NonZeroStiffnessMin],
					Dt,
					bStiffnessHasWeightMap,
					&Stiffness.GetIndices().GetData()[NonZeroStiffnessMin],
					&Stiffness.GetTable().GetData()[0],
					bVelocityStiffnessHasWeightMap,
					&VelocityTargetStiffness.GetIndices().GetData()[NonZeroStiffnessMin],
					&VelocityTargetStiffness.GetTable().GetData()[0],
					NonZeroStiffnessMax - NonZeroStiffnessMin 
				);
			}
			else
#endif
			{
				ispc::ApplyMultiResConstraintsWithWeightMaps(
					(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
					(const ispc::FVector3f*)Particles.XArray().GetData(),
					(const ispc::FVector3f*)FineTargetPositions.GetData(),
					(const ispc::FVector3f*)FineTargetVelocities.GetData(),
					Dt,
					bStiffnessHasWeightMap,
					Stiffness.GetIndices().GetData(),
					Stiffness.GetTable().GetData(),
					bVelocityStiffnessHasWeightMap,
					VelocityTargetStiffness.GetIndices().GetData(),
					VelocityTargetStiffness.GetTable().GetData(),
					Particles.Size()
				);
			}

		}
	}
	else
#endif
	{

		if (Stiffness.HasWeightMap())
		{
			if (VelocityTargetStiffness.HasWeightMap())
			{
				for (int32 Index = 0; Index < Particles.Size(); ++Index)
				{
					const FSolverReal ExpStiffness = Stiffness[Index];
					const FSolverReal ExpVelocityStiffness = VelocityTargetStiffness[Index];
					ApplyHelper(Particles, Dt, Index, ExpStiffness, ExpVelocityStiffness);
				}
			}
			else
			{
				const FSolverReal ExpVelocityStiffness = (FSolverReal)VelocityTargetStiffness;
				for (int32 Index = 0; Index < Particles.Size(); ++Index)
				{
					const FSolverReal ExpStiffness = Stiffness[Index];
					ApplyHelper(Particles, Dt, Index, ExpStiffness, ExpVelocityStiffness);
				}
			}
		}
		else
		{
			const FSolverReal ExpStiffness = (FSolverReal)Stiffness;
			if (VelocityTargetStiffness.HasWeightMap())
			{
				for (int32 Index = 0; Index < Particles.Size(); ++Index)
				{
					const FSolverReal ExpVelocityStiffness = VelocityTargetStiffness[Index];
					ApplyHelper(Particles, Dt, Index, ExpStiffness, ExpVelocityStiffness);
				}
			}
			else
			{
				const FSolverReal ExpVelocityStiffness = (FSolverReal)VelocityTargetStiffness;
				if (ExpStiffness == (FSolverReal)0.f && ExpVelocityStiffness == (FSolverReal)0.f)
				{
					return;
				}
				for (int32 Index = 0; Index < Particles.Size(); ++Index)
				{
					ApplyHelper(Particles, Dt, Index, ExpStiffness, ExpVelocityStiffness);
				}
			}
		}
	}
}

void FMultiResConstraints::UpdateFineTargets(const FSolverParticlesRange& CoarseParticles)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMultiResConstraints_UpdateFineTargets);

	CoarseBarycentricMass.SetNum(NumParticles);
	FineTargetPositions.SetNum(NumParticles);
	FineTargetVelocities.SetNum(NumParticles);
	TConstArrayView<FSolverVec3> CoarseX = CoarseParticles.XArray();
	TConstArrayView<FSolverReal> CoarseMass = CoarseParticles.GetM();
	TConstArrayView<FSolverVec3> CoarseVelocity = CoarseParticles.GetV();
	if (bMultiResConstraintApplyTargetNormalOffset)
	{
		TArray<FSolverVec3> Normals;
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FXPBDMultiResConstraints_CalculateNormals);
			Normals.SetNum(CoarseParticles.Size());
			// Need to calculate coarse normals
			TArray<FSolverVec3> FaceNormals;
			CoarseMesh.GetFaceNormals(FaceNormals, CoarseX, /*ReturnEmptyOnError =*/ false);
			CoarseMesh.GetPointNormals(TArrayView<FSolverVec3>(Normals), TConstArrayView<FSolverVec3>(FaceNormals), /*bUseGlobalArray =*/ false);
		}

#if INTEL_ISPC
		if (!bUseXPBD && bChaos_MultiRes_ISPC_Enabled)
		{
#if !UE_BUILD_SHIPPING
			if (Stiffness.HasWeightMap() && bChaos_MultiRes_SparseWeightMap_Enabled && bStiffnessEntriesInitialized)
			{
				ispc::MultiResUpdateFineTargets(
					(ispc::FVector3f*)&FineTargetPositions.GetData()[NonZeroStiffnessMin],
					(ispc::FVector3f*)&FineTargetVelocities.GetData()[NonZeroStiffnessMin],
					(const ispc::FVector3f*)CoarseX.GetData(),
					(const ispc::FVector3f*)Normals.GetData(),
					(const ispc::FVector3f*)CoarseVelocity.GetData(),
					(const ispc::FVector4f*)&CoarseToFinePositionBaryCoordsAndDist.GetData()[NonZeroStiffnessMin],
					(const ispc::FIntVector*)&CoarseToFineSourceMeshVertIndices.GetData()[NonZeroStiffnessMin],
					NonZeroStiffnessMax - NonZeroStiffnessMin
				);
			}
			else
#endif
			{
				ispc::MultiResUpdateFineTargets(
					(ispc::FVector3f*)FineTargetPositions.GetData(),
					(ispc::FVector3f*)FineTargetVelocities.GetData(),
					(const ispc::FVector3f*)CoarseX.GetData(),
					(const ispc::FVector3f*)Normals.GetData(),
					(const ispc::FVector3f*)CoarseVelocity.GetData(),
					(const ispc::FVector4f*)CoarseToFinePositionBaryCoordsAndDist.GetData(),
					(const ispc::FIntVector*)CoarseToFineSourceMeshVertIndices.GetData(),
					NumParticles
				);
			}
		}
		else
#endif
		{
			for (int32 Index = 0; Index < NumParticles; ++Index)
			{
				const TVec4<FSolverReal>& PositionBaryCoordsAndDist = CoarseToFinePositionBaryCoordsAndDist[Index];
				const TVec3<int32>& SourceMeshVertIndices = CoarseToFineSourceMeshVertIndices[Index];
				FineTargetPositions[Index] =
					PositionBaryCoordsAndDist[0] * (CoarseX[SourceMeshVertIndices[0]] + Normals[SourceMeshVertIndices[0]] * PositionBaryCoordsAndDist[3]) +
					PositionBaryCoordsAndDist[1] * (CoarseX[SourceMeshVertIndices[1]] + Normals[SourceMeshVertIndices[1]] * PositionBaryCoordsAndDist[3]) +
					PositionBaryCoordsAndDist[2] * (CoarseX[SourceMeshVertIndices[2]] + Normals[SourceMeshVertIndices[2]] * PositionBaryCoordsAndDist[3]);
				CoarseBarycentricMass[Index] =
					PositionBaryCoordsAndDist[0] * CoarseMass[SourceMeshVertIndices[0]] +
					PositionBaryCoordsAndDist[1] * CoarseMass[SourceMeshVertIndices[1]] +
					PositionBaryCoordsAndDist[2] * CoarseMass[SourceMeshVertIndices[2]];
				FineTargetVelocities[Index] =
					PositionBaryCoordsAndDist[0] * CoarseVelocity[SourceMeshVertIndices[0]] +
					PositionBaryCoordsAndDist[1] * CoarseVelocity[SourceMeshVertIndices[1]] +
					PositionBaryCoordsAndDist[2] * CoarseVelocity[SourceMeshVertIndices[2]];
			}
		}
	}
	else
	{
		for (int32 Index = 0; Index < NumParticles; ++Index)
		{
			const TVec4<FSolverReal>& PositionBaryCoordsAndDist = CoarseToFinePositionBaryCoordsAndDist[Index];
			const TVec3<int32>& SourceMeshVertIndices = CoarseToFineSourceMeshVertIndices[Index];
			FineTargetPositions[Index] =
				PositionBaryCoordsAndDist[0] * CoarseX[SourceMeshVertIndices[0]] +
				PositionBaryCoordsAndDist[1] * CoarseX[SourceMeshVertIndices[1]] +
				PositionBaryCoordsAndDist[2] * CoarseX[SourceMeshVertIndices[2]];
			CoarseBarycentricMass[Index] =
				PositionBaryCoordsAndDist[0] * CoarseMass[SourceMeshVertIndices[0]] +
				PositionBaryCoordsAndDist[1] * CoarseMass[SourceMeshVertIndices[1]] +
				PositionBaryCoordsAndDist[2] * CoarseMass[SourceMeshVertIndices[2]];
			FineTargetVelocities[Index] =
				PositionBaryCoordsAndDist[0] * CoarseVelocity[SourceMeshVertIndices[0]] +
				PositionBaryCoordsAndDist[1] * CoarseVelocity[SourceMeshVertIndices[1]] +
				PositionBaryCoordsAndDist[2] * CoarseVelocity[SourceMeshVertIndices[2]];
		}
	}
}
} // End namespace Chaos::Softs
