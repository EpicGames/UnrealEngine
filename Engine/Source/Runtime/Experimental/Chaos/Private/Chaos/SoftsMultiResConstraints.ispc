// Copyright Epic Games, Inc. All Rights Reserved.

#define EXPLICIT_VECTOR4 1

#include "Math/Vector.isph"
#include "Chaos/PBDSofts.isph"

export void ApplyMultiResConstraints(uniform FVector4f PandInvM[],
									const uniform FVector3f X[],
									const uniform FVector3f TargetPositions[],
									const uniform FVector3f TargetVelocities[],
									const uniform float Dt,
									const uniform float Stiffness,
									const uniform float VelocityStiffness,
									const uniform int32 NumParticles)
{
	foreach(Index = 0 ... NumParticles)
	{
		const uniform int32 Index1 = extract(Index,0);
		const varying FVector4f PandInvM1 = VectorLoad(&PandInvM[Index1]);

		varying FVector3f P1;
		varying float InvM1;
		UnzipPandInvM(PandInvM1, P1, InvM1);
		
		if(InvM1 > 0.f)
		{
			const varying FVector3f X1 = VectorLoad(&X[Index1]);
			const varying FVector3f TargetPos = VectorLoad(&TargetPositions[Index1]);
			const varying FVector3f TargetVel = VectorLoad(&TargetVelocities[Index1]);

			P1 = P1 - Stiffness * (P1 - TargetPos);
			const varying FVector3f FineDisplacement = P1 - X1;
			P1 = P1 - VelocityStiffness * (FineDisplacement - TargetVel * Dt);
			VectorStore(&PandInvM[Index1], SetVector4(P1, InvM1));
		}
	}
}

export void ApplyMultiResConstraintsWithWeightMaps(uniform FVector4f PandInvM[],
									const uniform FVector3f X[],
									const uniform FVector3f TargetPositions[],
									const uniform FVector3f TargetVelocities[],
									const uniform float Dt,
									const uniform bool StiffnessHasMap,
									const uniform uint8 StiffnessIndices[],
									const uniform float StiffnessTable[],
									const uniform bool VelocityStiffnessHasMap,
									const uniform uint8 VelocityStiffnessIndices[],
									const uniform float VelocityStiffnessTable[],
									const uniform int32 NumParticles)
{
	foreach(Index = 0 ... NumParticles)
	{
		const uniform int32 Index1 = extract(Index,0);
		const varying FVector4f PandInvM1 = VectorLoad(&PandInvM[Index1]);

		varying FVector3f P1;
		varying float InvM1;
		UnzipPandInvM(PandInvM1, P1, InvM1);
		
		if(InvM1 > 0.f)
		{
			const varying int8 StiffnessIndex = StiffnessHasMap ? StiffnessIndices[Index] : 0;
			const varying int8 VelocityStiffnessIndex = VelocityStiffnessHasMap ? VelocityStiffnessIndices[Index] : 0;
			#pragma ignore warning(perf)
			const varying float Stiffness = StiffnessTable[StiffnessIndex];
			#pragma ignore warning(perf)
			const varying float VelocityStiffness = VelocityStiffnessTable[VelocityStiffnessIndex];

			const varying FVector3f X1 = VectorLoad(&X[Index1]);
			const varying FVector3f TargetPos = VectorLoad(&TargetPositions[Index1]);
			const varying FVector3f TargetVel = VectorLoad(&TargetVelocities[Index1]);

			P1 = P1 - Stiffness * (P1 - TargetPos);
			const varying FVector3f FineDisplacement = P1 - X1;
			P1 = P1 - VelocityStiffness * (FineDisplacement - TargetVel * Dt);
			VectorStore(&PandInvM[Index1], SetVector4(P1, InvM1));
		}
	}
}

export void MultiResUpdateFineTargets(uniform FVector3f FineTargetPositions[],
							uniform FVector3f FineTargetVelocities[],
							const uniform FVector3f CoarseX[],
							const uniform FVector3f CoarseNormals[],
							const uniform FVector3f CoarseV[],
							const uniform FVector4f CoarseToFinePositionBaryCoordsAndDist[],
							const uniform FIntVector CoarseToFineSourceMeshVertIndices[],
							const uniform int32 NumParticles)
{
	foreach(Index = 0 ... NumParticles)
	{
		const uniform int32 Index0 = extract(Index,0);

		const varying FVector4f PositionBaryCoordsAndDist = VectorLoad(&CoarseToFinePositionBaryCoordsAndDist[Index0]);
		const varying FIntVector SourceMeshVertIndices = VectorLoad(&CoarseToFineSourceMeshVertIndices[Index0]);
		const varying int32 Index1 = SourceMeshVertIndices.V[0];
		const varying int32 Index2 = SourceMeshVertIndices.V[1];
		const varying int32 Index3 = SourceMeshVertIndices.V[2];

		const varying FVector3f CoarseX1 = VectorGather(&CoarseX[Index1]);
		const varying FVector3f CoarseX2 = VectorGather(&CoarseX[Index2]);
		const varying FVector3f CoarseX3 = VectorGather(&CoarseX[Index3]);
		const varying FVector3f CoarseNormal1 = VectorGather(&CoarseNormals[Index1]);
		const varying FVector3f CoarseNormal2 = VectorGather(&CoarseNormals[Index2]);
		const varying FVector3f CoarseNormal3 = VectorGather(&CoarseNormals[Index3]);
		const varying FVector3f CoarseV1 = VectorGather(&CoarseV[Index1]);
		const varying FVector3f CoarseV2 = VectorGather(&CoarseV[Index2]);
		const varying FVector3f CoarseV3 = VectorGather(&CoarseV[Index3]);

		const varying FVector3f FineTargetPos = 
					PositionBaryCoordsAndDist.V[0] * (CoarseX1 + CoarseNormal1 * PositionBaryCoordsAndDist.V[3]) +
					PositionBaryCoordsAndDist.V[1] * (CoarseX2 + CoarseNormal2 * PositionBaryCoordsAndDist.V[3]) +
					PositionBaryCoordsAndDist.V[2] * (CoarseX3 + CoarseNormal3 * PositionBaryCoordsAndDist.V[3]);
		const varying FVector3f FineTargetVel =
					PositionBaryCoordsAndDist.V[0] * CoarseV1 +
					PositionBaryCoordsAndDist.V[1] * CoarseV2 +
					PositionBaryCoordsAndDist.V[2] * CoarseV3;
		VectorStore(&FineTargetPositions[Index0], FineTargetPos);
		VectorStore(&FineTargetVelocities[Index0], FineTargetVel);
	}
}
