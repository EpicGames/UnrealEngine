// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef __SOFTSSPRING_ISPH__
#define __SOFTSSPRING_ISPH__

#define EXPLICIT_VECTOR4 1

#include "Math/Vector.isph"
#include "Chaos/PBDSofts.isph"

inline float SafeNormalize(FVector3f &Direction)
{
	const float SizeSquared = VectorSizeSquared(Direction);
	const float Size = sqrt(SizeSquared);
	Direction = VectorSelect((SizeSquared < FLOAT_KINDA_SMALL_NUMBER), FloatForwardVector, Direction / Size);
	return (SizeSquared < FLOAT_KINDA_SMALL_NUMBER) ? FLOAT_ZERO : Size;
}

// Spring without damping
template <typename StiffnessFloatType>
inline FVector3f GetXPBDSpringDelta(const FVector3f& P1, const FVector3f& P2, const varying float InvM1, const varying float InvM2,
	 const uniform float Dt, const varying float Dist, const varying float Lambda, const StiffnessFloatType Stiffness, float& DLambda)
{
	FVector3f Direction = P1 - P2;
	const float Distance = SafeNormalize(Direction);
	const varying float Offset = Distance - Dist;
	
	const varying float AlphaInv = Stiffness * Dt * Dt;
	
	const float CombinedInvMass = InvM2 + InvM1;

	DLambda = (-AlphaInv * Offset - Lambda) / (AlphaInv * CombinedInvMass + (uniform float)1.f);

	return DLambda * Direction;
}

// This is a following the original XPBD paper using a single lambda for stretch and damping.
template <typename StiffnessFloatType>
inline FVector3f GetXPBDSpringDeltaWithDamping(const FVector3f& P1, const FVector3f& P2, const FVector3f& X1, const FVector3f& X2,
	const varying float InvM1, const varying float InvM2,
	const uniform float Dt, const varying float Dist, const varying float Lambda, const StiffnessFloatType Stiffness, const StiffnessFloatType DampingRatio, float& DLambda)
{
	FVector3f Direction = P1 - P2;
	const float Distance = SafeNormalize(Direction);
	const varying float Offset = Distance - Dist;
	const varying float OneOverDist = select(Dist< FLOAT_SMALL_NUMBER, 1.f, 1.f / Dist);
	
	const varying float AlphaInv = Stiffness * Dt * Dt;
	
	const float CombinedInvMass = InvM2 + InvM1;
	const varying float Damping = (uniform float)2.f * DampingRatio * sqrt(Stiffness / CombinedInvMass) * OneOverDist;
	const varying float BetaDt = Damping * Dt;
	const varying FVector3f RelativeVelocityTimesDt = P1 - X1 - P2 + X2;

	DLambda = (-AlphaInv * Offset - Lambda - BetaDt * VectorDot(Direction, RelativeVelocityTimesDt)) / ((AlphaInv + BetaDt)*CombinedInvMass + (uniform float)1.f);

	return DLambda * Direction;
}

// Spring damping constraint (separate from spring stretching)
template <typename StiffnessFloatType>
inline FVector3f GetXPBDSpringDampingDelta(const FVector3f& P1, const FVector3f& P2, const FVector3f& X1, const FVector3f& X2,
	const varying float InvM1, const varying float InvM2,
	const uniform float Dt, const varying float Dist, const varying float Lambda, const StiffnessFloatType Stiffness, const StiffnessFloatType DampingRatio, float& DLambda)
{
	FVector3f Direction = P1 - P2;
	const float Distance = SafeNormalize(Direction);
	const varying float Offset = Distance - Dist;
	const varying float OneOverDist = select(Dist< FLOAT_SMALL_NUMBER, 1.f, 1.f / Dist);
	
	const float CombinedInvMass = InvM2 + InvM1;
	const varying float Damping = (uniform float)2.f * DampingRatio * sqrt(Stiffness / CombinedInvMass) * OneOverDist;
	const varying float BetaDt = Damping * Dt;
	const varying FVector3f RelativeVelocityTimesDt = P1 - X1 - P2 + X2;

	DLambda = ( -BetaDt * VectorDot(Direction, RelativeVelocityTimesDt) - Lambda) / (BetaDt * CombinedInvMass + (uniform float)1.f);

	return DLambda * Direction;
}

// Spring without damping
template <typename StiffnessFloatType>
inline FVector3f GetXPBDAxialSpringDelta(const FVector3f& P1, const FVector3f& P2, const FVector3f& P3, const varying float InvM1, const varying float InvM2, const varying float InvM3,
	 const uniform float Dt, const varying float Bary, const varying float Dist, const varying float Lambda, const StiffnessFloatType Stiffness, float& DLambda)
{
	FVector3f P = (P2 - P3) * Bary + P3;
	FVector3f Direction = P1 - P;
	const float Distance = SafeNormalize(Direction);
	const varying float Offset = Distance - Dist;
	
	const varying float AlphaInv = Stiffness * Dt * Dt;
	
	const float CombinedInvMass = (1.f - Bary) * InvM3 + Bary * InvM2 + InvM1;

	DLambda = (-AlphaInv * Offset - Lambda) / (AlphaInv * CombinedInvMass + (uniform float)1.f);

	return DLambda * Direction;
}

// This is a following the original XPBD paper using a single lambda for stretch and damping.
template <typename StiffnessFloatType>
inline FVector3f GetXPBDAxialSpringDeltaWithDamping(const FVector3f& P1, const FVector3f& P2, const FVector3f& P3, 
	const FVector3f& X1, const FVector3f& X2, const FVector3f& X3,
	const varying float InvM1, const varying float InvM2, const varying float InvM3,
	const uniform float Dt, const varying float Bary, const varying float Dist, const varying float Lambda, 
	const StiffnessFloatType Stiffness, const StiffnessFloatType DampingRatio, float& DLambda)
{
	FVector3f P = (P2 - P3) * Bary + P3;
	FVector3f Direction = P1 - P;
	const float Distance = SafeNormalize(Direction);
	const varying float Offset = Distance - Dist;
	const varying float OneOverDist = select(Dist < FLOAT_SMALL_NUMBER, 1.f, 1.f / Dist);
	
	const varying float AlphaInv = Stiffness * Dt * Dt;
	
	const float CombinedInvMass = (1.f - Bary) * InvM3 + Bary * InvM2 + InvM1;
	const varying float Damping = (uniform float)2.f * DampingRatio * sqrt(Stiffness / CombinedInvMass) * OneOverDist;
	const varying float BetaDt = Damping * Dt;
	const varying FVector3f RelativeVelocityTimesDt = P1 - X1 - P2 + X2;

	DLambda = (-AlphaInv * Offset - Lambda - BetaDt * VectorDot(Direction, RelativeVelocityTimesDt)) / ((AlphaInv + BetaDt)*CombinedInvMass + (uniform float)1.f);

	return DLambda * Direction;
}

// Spring damping constraint (separate from spring stretching)
template <typename StiffnessFloatType>
inline FVector3f GetXPBDAxialSpringDampingDelta(const FVector3f& P1, const FVector3f& P2, const FVector3f& P3, 
	const FVector3f& X1, const FVector3f& X2, const FVector3f& X3,
	const varying float InvM1, const varying float InvM2, const varying float InvM3,
	const uniform float Dt, const varying float Bary, const varying float Dist, const varying float Lambda, 
	const StiffnessFloatType Stiffness, const StiffnessFloatType DampingRatio, float& DLambda)
{
	FVector3f P = (P2 - P3) * Bary + P3;
	FVector3f Direction = P1 - P;
	const float Distance = SafeNormalize(Direction);
	const varying float Offset = Distance - Dist;
	const varying float OneOverDist = select(Dist < FLOAT_SMALL_NUMBER, 1.f, 1.f / Dist);
	
	const float CombinedInvMass = (1.f - Bary) * InvM3 + Bary * InvM2 + InvM1;
	const varying float Damping = (uniform float)2.f * DampingRatio * sqrt(Stiffness / CombinedInvMass) * OneOverDist;
	const varying float BetaDt = Damping * Dt;
	const varying FVector3f RelativeVelocityTimesDt = P1 - X1 - P2 + X2;

	DLambda = (- BetaDt * VectorDot(Direction, RelativeVelocityTimesDt) - Lambda ) / (BetaDt*CombinedInvMass + (uniform float)1.f);

	return DLambda * Direction;
}
template <typename StiffnessFloatType>
inline FVector3f GetXPBDEmbeddedSpringDelta(const FVector3f& P1, const FVector3f& P2, const varying float InvM1, const varying float InvM2,
	 const uniform float Dt, const varying float Dist, const varying float Lambda, const StiffnessFloatType ExtensionStiffness, const StiffnessFloatType CompressionStiffness, float& DLambda)
{
	FVector3f Direction = P1 - P2;
	const float Distance = SafeNormalize(Direction);
	const varying float Offset = Distance - Dist;
	
	const varying float Stiffness = select(Offset >= 0, ExtensionStiffness, CompressionStiffness);
	const varying float AlphaInv = Stiffness * Dt * Dt;
	
	const float CombinedInvMass = InvM2 + InvM1;

	DLambda = (-AlphaInv * Offset - Lambda) / (AlphaInv * CombinedInvMass + (uniform float)1.f);

	return DLambda * Direction;
}


// Spring damping constraint (separate from spring stretching)
template <typename StiffnessFloatType>
inline FVector3f GetXPBDEmbeddedSpringDampingDelta(const FVector3f& P1, const FVector3f& P2, const FVector3f& X1, const FVector3f& X2,
	const varying float InvM1, const varying float InvM2,
	const uniform float Dt, const varying float Dist, const varying float Lambda, const StiffnessFloatType ExtensionStiffness, const StiffnessFloatType CompressionStiffness, const StiffnessFloatType DampingRatio, float& DLambda)
{
	FVector3f Direction = P1 - P2;
	const float Distance = SafeNormalize(Direction);
	const varying float Offset = Distance - Dist;
	const varying float OneOverDist = select(Dist< FLOAT_SMALL_NUMBER, 1.f, 1.f / Dist);
	
	const float CombinedInvMass = InvM2 + InvM1;
	const varying float Stiffness = select(Offset >= 0, ExtensionStiffness, CompressionStiffness);
	const varying float Damping = (uniform float)2.f * DampingRatio * sqrt(Stiffness / CombinedInvMass) * OneOverDist;
	const varying float BetaDt = Damping * Dt;
	const varying FVector3f RelativeVelocityTimesDt = P1 - X1 - P2 + X2;

	DLambda = ( -BetaDt * VectorDot(Direction, RelativeVelocityTimesDt) - Lambda) / (BetaDt * CombinedInvMass + (uniform float)1.f);

	return DLambda * Direction;
}
#endif
