// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/ISpatialAcceleration.h"

#include "Chaos/AABBTree.h"
#include "Chaos/BoundingVolume.h"
#include "Chaos/ParticleHandle.h"

namespace Chaos
{
	template<ESpatialAcceleration SpatialType, typename TPayloadType, typename T, int d>
	ISpatialAcceleration<TPayloadType, T, d>* TSpatialAccelerationSerializationFactory<SpatialType, TPayloadType, T, d>::Create()
	{
		if constexpr (SpatialType == ESpatialAcceleration::BoundingVolume)
		{
			return new TBoundingVolume<TPayloadType, T, d>();
		}
		else if constexpr (SpatialType == ESpatialAcceleration::AABBTree)
		{
			return new TAABBTree<TPayloadType, TAABBTreeLeafArray<TPayloadType>>();
		}
		else if constexpr (SpatialType == ESpatialAcceleration::AABBTreeBV)
		{
			return new TAABBTree<TPayloadType, TBoundingVolume<TPayloadType, T, 3>>();
		}
		else
		{
			return nullptr;
		}
	}

	#if !UE_MERGED_MODULES
	template class CHAOS_API Chaos::ISpatialAcceleration<int32, FReal, 3>;
	template class CHAOS_API Chaos::ISpatialAcceleration<FAccelerationStructureHandle, FReal, 3>;
	template class CHAOS_API Chaos::ISpatialVisitor<int32, FReal>;
	template class CHAOS_API Chaos::ISpatialVisitor<FAccelerationStructureHandle, FReal>;
	#endif

	template class Chaos::ISpatialAcceleration<FAccelerationStructureHandle, FReal, 3>* TSpatialAccelerationSerializationFactory<ESpatialAcceleration::BoundingVolume, FAccelerationStructureHandle, FReal, 3>::Create();
	template class Chaos::ISpatialAcceleration<FAccelerationStructureHandle, FReal, 3>* TSpatialAccelerationSerializationFactory<ESpatialAcceleration::AABBTree, FAccelerationStructureHandle, FReal, 3>::Create();
	template class Chaos::ISpatialAcceleration<FAccelerationStructureHandle, FReal, 3>* TSpatialAccelerationSerializationFactory<ESpatialAcceleration::AABBTreeBV, FAccelerationStructureHandle, FReal, 3>::Create();
}