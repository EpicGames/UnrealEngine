// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/SpatialHash.h"
#include "ProfilingDebugging/ScopedTimers.h"

namespace Chaos
{
	DEFINE_LOG_CATEGORY_STATIC(LogChaosSpatialHash, Verbose, All);


	template<class T>
	void TSpatialHash<T>::Init(const T Radius)
	{
		double Time = 0.0;
		FDurationTimer Timer(Time);

		MCellSize = (T)2.0 * Radius;

		MBoundingBox = TAABB<T, 3>(TVector<T, 3>(0.0), Chaos::TVector<T, 3>(0.0));
		for (int32 Idx = 0; Idx < MParticles.Num(); ++Idx)
		{
			MBoundingBox.GrowToInclude(MParticles[Idx]);
		}
		TVector<T, 3> Extents = MBoundingBox.Extents();

		// MCellSize needs to be smaller than smallest axis length
		T SmallestAxisLength = Extents[SmallestAxis()];
		if (MCellSize > SmallestAxisLength)
		{
			MCellSize = SmallestAxisLength / (T)2.0;
		}

		T PrincipalAxisLength = Extents[MBoundingBox.LargestAxis()];
		int32 NumberOfCellsOnPrincipalAxis = FMath::CeilToInt32(PrincipalAxisLength / MCellSize);
		MCellSize = PrincipalAxisLength / (T)NumberOfCellsOnPrincipalAxis;
		T CellSizeInv = (T)1.0 / MCellSize;

		MNumberOfCellsX = FMath::CeilToInt32(Extents[0] * CellSizeInv) + 1;
		MNumberOfCellsY = FMath::CeilToInt32(Extents[1] * CellSizeInv) + 1;
		MNumberOfCellsZ = FMath::CeilToInt32(Extents[2] * CellSizeInv) + 1;

		for (int32 IdxParticle = 0; IdxParticle < MParticles.Num(); ++IdxParticle)
		{
			int32 HashTableIdx = HashFunction(MParticles[IdxParticle]);
			ensure(HashTableIdx < MNumberOfCellsX * MNumberOfCellsY * MNumberOfCellsZ);
			if (MHashTable.Contains(HashTableIdx))
			{
				MHashTable[HashTableIdx].Add(IdxParticle);
			}
			else
			{
				MHashTable.Add(HashTableIdx);
				MHashTable[HashTableIdx].Add(IdxParticle);
			}
		}

		Timer.Stop();
		UE_LOGF(LogChaosSpatialHash, Log, "TSpatialHash<T>::Init() Time is %f", Time);
	}					

	template<class T>
	void TSpatialHash<T>::Init()
	{
		double Time = 0.0;
		FDurationTimer Timer(Time);

		MBoundingBox = TAABB<T, 3>(TVector<T, 3>(0.0), Chaos::TVector<T, 3>(0.0));
		for (int32 Idx = 0; Idx < MParticles.Num(); ++Idx)
		{
			MBoundingBox.GrowToInclude(MParticles[Idx]);
		}
		TVector<T, 3> Extents = MBoundingBox.Extents();

		// MCellSize needs to be smaller than smallest axis length
		T SmallestAxisLength = Extents[SmallestAxis()];
		MCellSize = SmallestAxisLength / (T)10.0;

		T PrincipalAxisLength = Extents[MBoundingBox.LargestAxis()];
		int32 NumberOfCellsOnPrincipalAxis = FMath::CeilToInt32(PrincipalAxisLength / MCellSize);
		MCellSize = PrincipalAxisLength / (T)NumberOfCellsOnPrincipalAxis;
		T CellSizeInv = (T)1.0 / MCellSize;

		MNumberOfCellsX = FMath::CeilToInt32(Extents[0] * CellSizeInv) + 1;
		MNumberOfCellsY = FMath::CeilToInt32(Extents[1] * CellSizeInv) + 1;
		MNumberOfCellsZ = FMath::CeilToInt32(Extents[2] * CellSizeInv) + 1;

		for (int32 IdxParticle = 0; IdxParticle < MParticles.Num(); ++IdxParticle)
		{
			int32 HashTableIdx = HashFunction(MParticles[IdxParticle]);
			ensure(HashTableIdx < MNumberOfCellsX * MNumberOfCellsY * MNumberOfCellsZ);
			if (MHashTable.Contains(HashTableIdx))
			{
				MHashTable[HashTableIdx].Add(IdxParticle);
			}
			else
			{
				MHashTable.Add(HashTableIdx);
				MHashTable[HashTableIdx].Add(IdxParticle);
			}
		}

		Timer.Stop();
		UE_LOGF(LogChaosSpatialHash, Log, "TSpatialHash<T>::Init() Time is %f", Time);
	}

	template<class T>
	void TSpatialHash<T>::Update(const TArray<TVector<T, 3>>& Particles, const T Radius)
	{
		MParticles = Particles;
		MHashTable.Empty();
		Init(Radius);
	}

	template<class T>
	void TSpatialHash<T>::Update(const TArray<TVector<T, 3>>& Particles)
	{
		MParticles = Particles;
		MHashTable.Empty();
		Init();
	}

	template<class T>
	void TSpatialHash<T>::Update(const T Radius)
	{
		MHashTable.Empty();
		Init(Radius);
	}

	template <class T>
	TArray<int32> TSpatialHash<T>::GetClosestPoints(const TVector<T, 3>& Particle, const T MaxRadius)
	{
		double Time = 0.0;
		FDurationTimer Timer(Time);

		TSet<int32> ClosestPoints;
		int32 MaxN = ComputeMaxN(Particle, MaxRadius);

		for (int32 IdxRing = 0; IdxRing < MaxN; ++IdxRing)
		{
			TSet<int32> CellIndices = GetNRing(Particle, IdxRing);
			for (auto& CellIdx : CellIndices)
			{
				if (MHashTable.Contains(CellIdx))
				{
					ClosestPoints.Append(MHashTable[CellIdx]);
				}
			}
		}

		// Need to delete points which are out of MaxRadius range
		TSet<int32> PointsToRemove;
		T MaxRadiusSquared = MaxRadius * MaxRadius;
		for (auto& Elem : ClosestPoints)
		{
			TVector<T, 3> Diff = Particle - MParticles[Elem];
			if (Diff.SizeSquared() > MaxRadiusSquared)
			{
				PointsToRemove.Add(Elem);
			}
		}
		if (PointsToRemove.Num() > 0)
		{
			ClosestPoints = ClosestPoints.Difference(ClosestPoints.Intersect(PointsToRemove));
		}

		Timer.Stop();
		UE_LOGF(LogChaosSpatialHash, Log, "TSpatialHash<T>::GetClosestPoints() Time is %f", Time);

		return ClosestPoints.Array();
	}

	template<class T>
	TArray<int32> TSpatialHash<T>::GetClosestPoints(const TVector<T, 3>& Particle, const T MaxRadius, const int32 MaxPoints)
	{
		double Time = 0.0;
		FDurationTimer Timer(Time);

		TSet<int32> ClosestPoints;
		int32 MaxN = ComputeMaxN(Particle, MaxRadius);

		for (int32 IdxRing = 0; IdxRing < MaxN; ++IdxRing)
		{
			TSet<int32> CellIndices = GetNRing(Particle, IdxRing);
			for (auto& CellIdx : CellIndices)
			{
				if (MHashTable.Contains(CellIdx))
				{
					ClosestPoints.Append(MHashTable[CellIdx]);
				}
			}
		}

		// Need to delete points which are out of MaxRadius range
		TSet<int32> PointsToRemove;
		T MaxRadiusSquared = MaxRadius * MaxRadius;
		for (auto& Elem : ClosestPoints)
		{
			TVector<T, 3> Diff = Particle - MParticles[Elem];
			if (Diff.SizeSquared() > MaxRadiusSquared)
			{
				PointsToRemove.Add(Elem);
			}
		}
		if (PointsToRemove.Num() > 0)
		{
			ClosestPoints = ClosestPoints.Difference(ClosestPoints.Intersect(PointsToRemove));
		}

		// Sort ClosestPoints
		TMap<int32, T> ParticleIdxDistanceMap;
		for (auto& Elem : ClosestPoints)
		{
			TVector<T, 3> Diff = Particle - MParticles[Elem];
			ParticleIdxDistanceMap.Add(Elem, Diff.SizeSquared());
		}
		ParticleIdxDistanceMap.ValueSort([](const T& Distance1, const T& Distance2) {
			return Distance1 < Distance2;
		});

		TArray<int32> ClosestPointsArray;
		ParticleIdxDistanceMap.GetKeys(ClosestPointsArray);

		// Delete points after MaxPoints
		if (ClosestPointsArray.Num() > MaxPoints)
		{
			ClosestPointsArray.SetNum(MaxPoints);
		}
		
		Timer.Stop();
		UE_LOGF(LogChaosSpatialHash, Log, "TSpatialHash<T>::GetClosestPoints() Time is %f", Time);
			
		return ClosestPointsArray;
	}

	template<class T>
	int32 TSpatialHash<T>::GetClosestPoint(const TVector<T, 3>& Particle)
	{
		double Time = 0.0;
		FDurationTimer Timer(Time);

		int32 ClosestPointIdx = INDEX_NONE;

		if (MBoundingBox.Contains(Particle))
		{
			TSet<int32> ClosestPoints;

			int32 IdxRing = 0;
			while (ClosestPoints.Num() < 1)
			{
				TSet<int32> CellIndices = GetNRing(Particle, IdxRing);
				for (auto& CellIdx : CellIndices)
				{
					if (MHashTable.Contains(CellIdx))
					{
						ClosestPoints.Append(MHashTable[CellIdx]);
					}
				}

				IdxRing++;
			}

			ClosestPointIdx = ClosestPoints.Array()[0];

			// If ClosestPoints have more than one point find the closest one
			if (ClosestPoints.Num() > 1)
			{
				FReal DistanceSquared = TNumericLimits<FReal>::Max();
				for (auto& PointIdx : ClosestPoints)
				{
					FVec3 Diff = Particle - MParticles[PointIdx];
					T DiffSquared = Diff.SizeSquared();
					if (DiffSquared < DistanceSquared)
					{
						DistanceSquared = DiffSquared;
						ClosestPointIdx = PointIdx;
					}
				}
			}
		}
		else
		{
			TVector<T, 3> ClosestPointOnBox = MBoundingBox.FindClosestPoint(Particle);
			ClosestPointIdx = GetClosestPoint(ClosestPointOnBox);
		}

		Timer.Stop();
		UE_LOGF(LogChaosSpatialHash, Log, "TSpatialHash<T>::GetClosestPoint() Time is %f", Time);

		return ClosestPointIdx;
	}

	template<class T>
	int32 TSpatialHash<T>::ComputeMaxN(const TVector<T, 3>& Particle, const T Radius)
	{
		int32 MaxN = -INT_MAX;
		TArray<int32> IndexParticleArray; IndexParticleArray.SetNum(3);
		ComputeGridXYZ(Particle, IndexParticleArray[0], IndexParticleArray[1], IndexParticleArray[2]);

		TArray<TVector<T, 3>> Points;
		Points.Add(Particle - TVector<T, 3>(Radius, 0.0, 0.0));
		Points.Add(Particle + TVector<T, 3>(Radius, 0.0, 0.0));
		Points.Add(Particle - TVector<T, 3>(0.0, Radius, 0.0));
		Points.Add(Particle + TVector<T, 3>(0.0, Radius, 0.0));
		Points.Add(Particle - TVector<T, 3>(0.0, 0.0, Radius));
		Points.Add(Particle + TVector<T, 3>(0.0, 0.0, Radius));

		TArray<int32> IndexPointArray; IndexPointArray.SetNum(3);
		for (int32 IdxPoint = 0; IdxPoint < Points.Num(); ++IdxPoint)
		{
			ComputeGridXYZ(Points[IdxPoint], IndexPointArray[0], IndexPointArray[1], IndexPointArray[2]);
			IndexPointArray[0] = FMath::Clamp(IndexPointArray[0], 0, MNumberOfCellsX - 1);
			IndexPointArray[1] = FMath::Clamp(IndexPointArray[1], 0, MNumberOfCellsY - 1);
			IndexPointArray[2] = FMath::Clamp(IndexPointArray[2], 0, MNumberOfCellsZ - 1);
			for (int32 Idx = 0; Idx < 3; ++Idx)
			{
				int32 Diff = FMath::Abs(IndexParticleArray[Idx] - IndexPointArray[Idx]) + 1;
				if (Diff > MaxN)
				{
					MaxN = Diff;
				}
			}
		}

		return MaxN;
	}

	template<class T>
	TSet<int32> TSpatialHash<T>::GetNRing(const TVector<T, 3>& Particle, const int32 N)
	{
		TSet<int32> RingCells;

		int32 ParticleXIndex, ParticleYIndex, ParticleZIndex;
		ComputeGridXYZ(Particle, ParticleXIndex, ParticleYIndex, ParticleZIndex);

		int32 XIndex, YIndex, ZIndex;
		if (N == 0)
		{
			RingCells.Add(HashFunction(ParticleXIndex, ParticleYIndex, ParticleZIndex));
		}
		else
		{
			for (int32 XIdx = -N; XIdx <= N; ++XIdx)
			{
				for (int32 YIdx = -N; YIdx <= N; ++YIdx)
				{
					for (int32 ZIdx = -N; ZIdx <= N; ++ZIdx)
					{
						if (XIdx == N || XIdx == -N ||
							YIdx == N || YIdx == -N ||
							ZIdx == N || ZIdx == -N)
						{
							XIndex = ParticleXIndex + XIdx;
							YIndex = ParticleYIndex + YIdx;
							ZIndex = ParticleZIndex + ZIdx;

							if (XIndex >= 0 && XIndex < MNumberOfCellsX &&
								YIndex >= 0 && YIndex < MNumberOfCellsY &&
								ZIndex >= 0 && ZIndex < MNumberOfCellsZ)
							{
								RingCells.Add(HashFunction(XIndex, YIndex, ZIndex));
							}
						}
					}
				}
			}
		}

		return RingCells;
	}

	template<class T>
	void TSpatialHash<T>::ComputeGridXYZ(const TVector<T, 3>& Particle, int32& XIndex, int32& YIndex, int32& ZIndex)
	{
		T CellSizeInv = (T)1.0 / MCellSize;
		TVector<T, 3> Location = Particle - MBoundingBox.Min() + TVector<T, 3>((T)0.5 * MCellSize);
		XIndex = (int32)(Location.X * CellSizeInv);
		YIndex = (int32)(Location.Y * CellSizeInv);
		ZIndex = (int32)(Location.Z * CellSizeInv);
	}

	template<class T>
	int32 TSpatialHash<T>::HashFunction(const TVector<T, 3>& Particle)
	{
		int32 XIndex, YIndex, ZIndex;
		ComputeGridXYZ(Particle, XIndex, YIndex, ZIndex);

		return XIndex + YIndex * MNumberOfCellsX + ZIndex * MNumberOfCellsX * MNumberOfCellsY;
	}

	template<class T>
	int32 TSpatialHash<T>::HashFunction(const int32 XIndex, const int32 YIndex, const int32 ZIndex)
	{
		return XIndex + YIndex * MNumberOfCellsX + ZIndex * MNumberOfCellsX * MNumberOfCellsY;
	}

	template class TSpatialHash<FReal>;
}

