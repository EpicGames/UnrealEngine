// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/StrainModification.h"
#include "Chaos/PBDRigidClustering.h"
#include "PhysicsProxy/GeometryCollectionPhysicsProxy.h"
#include "PhysicsProxy/ClusterUnionPhysicsProxy.h"
#include "GeometryCollection/GeometryCollectionSimulationCoreTypes.h"
#include "GeometryCollection/GeometryCollection.h"

namespace
{
	float GetMaxAppliedStrain(const Chaos::FPBDRigidClusteredParticleHandle* Handle, const uint8 StrainTypes)
	{
		const float CollisionImpulses = (StrainTypes & Chaos::EStrainTypes::CollisionStrain) ? Handle->CollisionImpulses() : 0.f;
		const float ExternalStrain = (StrainTypes & Chaos::EStrainTypes::ExternalStrain) ? Handle->GetExternalStrain() : 0.f;
		return FMath::Max(CollisionImpulses, ExternalStrain);
	}
}

FGeometryCollectionPhysicsProxy* Chaos::FStrainedProxyAndRoot::CastToGeometryCollectionProxy() const
{
	if (Proxy && Proxy->GetType() == EPhysicsProxyType::GeometryCollectionType)
	{
		return static_cast<FGeometryCollectionPhysicsProxy*>(Proxy);
	}
	return nullptr;
}

bool Chaos::FStrainedProxyAndRoot::IsPartialDestruction() const
{
	return bPartialDestruction;
}

bool Chaos::FStrainedProxyAndRoot::operator==(const  Chaos::FStrainedProxyAndRoot& Other) const
{
	// no need to test the partial destruction flag 
	return ((Proxy == Other.Proxy) && (ParticleHandle == Other.ParticleHandle));
}


const IPhysicsProxyBase* Chaos::FStrainedProxyModifier::GetProxy() const
{
	return ProxyAndRoot.Proxy;
}

const Chaos::FPBDRigidParticleHandle* Chaos::FStrainedProxyModifier::GetOriginalRootHandle() const
{
	if (FGeometryCollectionPhysicsProxy* ProxyGC = ProxyAndRoot.CastToGeometryCollectionProxy())
	{
		return ProxyGC->GetInitialRootHandle_Internal();
	}
	return nullptr;
}

const Chaos::FPBDRigidParticleHandle* Chaos::FStrainedProxyModifier::GetParticleHandle() const
{
	return ProxyAndRoot.ParticleHandle;
}

template <typename TFunction>
static void ForEachRootChildParticle(const Chaos::FStrainedProxyAndRoot& ProxyAndRoot, const TSet<int32>* RestChildren, TFunction Func)
{
	if (ProxyAndRoot.IsPartialDestruction())
	{
		// when partial destruction, only process the particle handle
		Func(ProxyAndRoot.ParticleHandle);
	}
	else if (RestChildren)
	{
		if (FGeometryCollectionPhysicsProxy* ProxyGC = ProxyAndRoot.CastToGeometryCollectionProxy())
		{
			for (int32 RestChildIdx : *RestChildren)
			{
				Chaos::FPBDRigidClusteredParticleHandle* ChildHandle = ProxyGC->GetParticle_Internal(RestChildIdx);
				if ((ChildHandle == nullptr) || (ChildHandle->Parent() == nullptr)) { continue; }

				Func(ChildHandle);
			}
		}
	}
}

int32 Chaos::FStrainedProxyModifier::GetNumRestBreakables() const
{
	if (RestChildren)
	{
		return RestChildren->Num();
	}
	return 1; // cluster union case
}

int32 Chaos::FStrainedProxyModifier::GetNumBreakingStrains(const bool bDoubleCount, const uint8 StrainTypes) const
{
	int32 NumBreakingStrains = 0;
	ForEachRootChildParticle(ProxyAndRoot, RestChildren,
		[this, &NumBreakingStrains, &bDoubleCount, &StrainTypes] (Chaos::FPBDRigidClusteredParticleHandle* ChildHandle)
		{
			// Get the applied and internal strain
			const float InternalStrain = ChildHandle->GetInternalStrains();
			const float MaxAppliedStrain = GetMaxAppliedStrain(ChildHandle, StrainTypes);

			if (bDoubleCount && InternalStrain > SMALL_NUMBER)
			{
				NumBreakingStrains += (int32)(MaxAppliedStrain / InternalStrain);
			}
			else if (MaxAppliedStrain >= InternalStrain)
			{
				++NumBreakingStrains;
			}
		});

	// Return the number of breaking strains
	return NumBreakingStrains;
}

float Chaos::FStrainedProxyModifier::GetMaxBreakStrainRatio(const float FatigueThresholdPercent, const float FatigueThresholdMinimum, bool bRelative, const uint8 StrainTypes) const
{
	float MaxBreakStrainRatio = 0.f;
	ForEachRootChildParticle(ProxyAndRoot, RestChildren,
		[this, &MaxBreakStrainRatio, FatigueThresholdPercent, FatigueThresholdMinimum, bRelative, StrainTypes] (Chaos::FPBDRigidClusteredParticleHandle* ChildHandle)
		{
			// compute the strain ratio
			const float InternalStrain = ChildHandle->GetInternalStrains();
			const float MaxAppliedStrain = GetMaxAppliedStrain(ChildHandle, StrainTypes);
			const float FatigueThreshold = FMath::Max(FatigueThresholdMinimum, (FatigueThresholdPercent * InternalStrain));
			if (MaxAppliedStrain >= FatigueThreshold)
			{
				const float StrainRange = bRelative ? FMath::Max(0.f, InternalStrain - FatigueThreshold): InternalStrain;
				const float Strain = bRelative ? FMath::Max(0.f, MaxAppliedStrain - FatigueThreshold): MaxAppliedStrain;
				const float StrainRatio = (StrainRange > SMALL_NUMBER) ? (Strain / StrainRange) : 1.0f;
				MaxBreakStrainRatio = FMath::Max(MaxBreakStrainRatio, StrainRatio);
			}
		});

	return MaxBreakStrainRatio;
}

float Chaos::FStrainedProxyModifier::GetStrainUsedForBreakRatio(const float FatigueThresholdPercent, const float FatigueThresholdMinimum, bool bRelative, const uint8 StrainTypes)
{
	float MaxBreakStrainRatio = 0.f;
	float StrainForBreakRatio = 0.f;
	ForEachRootChildParticle(ProxyAndRoot, RestChildren,
		[this, &MaxBreakStrainRatio, &StrainForBreakRatio, FatigueThresholdPercent, FatigueThresholdMinimum, bRelative, StrainTypes](Chaos::FPBDRigidClusteredParticleHandle* ChildHandle)
		{
			// compute the strain ratio
			const float InternalStrain = ChildHandle->GetInternalStrains();
			const float MaxAppliedStrain = GetMaxAppliedStrain(ChildHandle, StrainTypes);
			const float FatigueThreshold = FMath::Max(FatigueThresholdMinimum, (FatigueThresholdPercent * InternalStrain));
			if (MaxAppliedStrain >= FatigueThreshold)
			{
				const float StrainRange = bRelative ? FMath::Max(0.f, InternalStrain - FatigueThreshold) : InternalStrain;
				const float Strain = bRelative ? FMath::Max(0.f, MaxAppliedStrain - FatigueThreshold) : MaxAppliedStrain;
				const float StrainRatio = (StrainRange > SMALL_NUMBER) ? (Strain / StrainRange) : 1.0f;
				if (StrainRatio > MaxBreakStrainRatio)
				{
					MaxBreakStrainRatio = StrainRatio;
					StrainForBreakRatio = MaxAppliedStrain;
				}
			}
		});

	return StrainForBreakRatio;
}

void Chaos::FStrainedProxyModifier::AdjustStrainForBreak(const float FatigueThresholdPercent, const float FatigueThresholdMinimum, const uint8 StrainTypes)
{
	ForEachRootChildParticle(ProxyAndRoot, RestChildren,
		[this, FatigueThresholdPercent, FatigueThresholdMinimum, StrainTypes] (Chaos::FPBDRigidClusteredParticleHandle* ChildHandle)
		{
			// compute the strain ratio
			const float InternalStrain = ChildHandle->GetInternalStrains();
			const float MaxAppliedStrain = GetMaxAppliedStrain(ChildHandle, StrainTypes);
			const float FatigueThreshold = FMath::Max(FatigueThresholdMinimum, (FatigueThresholdPercent * InternalStrain));
			if (MaxAppliedStrain >= FatigueThreshold)
			{
				RigidClustering.SetExternalStrain(ChildHandle, InternalStrain);
			}
		});
}

void Chaos::FStrainedProxyModifier::ClearStrains()
{
	ForEachRootChildParticle(ProxyAndRoot, RestChildren,
		[this] (Chaos::FPBDRigidClusteredParticleHandle* ChildHandle)
		{
			// Clear accumulated collision impulses
			ChildHandle->ClearCollisionImpulse();

			// Clear accumulated external strains
			//
			// NOTE: The following is deprecated:
			// ChildHandle->ClearExternalStrain();
			// Instead we use:
			RigidClustering.SetExternalStrain(ChildHandle, FReal(0));
		});
}

const TSet<int32>* Chaos::FStrainedProxyModifier::InitRestChildren(FGeometryCollectionPhysicsProxy* ProxyGC)
{
	if (ProxyGC == nullptr) { return nullptr; }

	// Get the root index of the proxy
	FSimulationParameters& Parameters = ProxyGC->GetSimParameters();
	const int32 RootIndex = Parameters.InitialRootIndex;
	if (RootIndex == INDEX_NONE) { return nullptr; }

	// Return a ptr to the set of child indices
	return &Parameters.RestCollectionShared->Children[RootIndex];
}

Chaos::FStrainedProxyModifier Chaos::FStrainedProxyIterator::operator*()
{
	return Chaos::FStrainedProxyModifier(RigidClustering, ProxyAndRoots[Index]);
}

Chaos::FStrainedProxyIterator& Chaos::FStrainedProxyIterator::operator++()
{
	++Index;
	return *this;
}

bool Chaos::FStrainedProxyIterator::operator==(const FStrainedProxyIterator& Other) const
{
	return
		ProxyAndRoots == Other.ProxyAndRoots &&
		Index == Other.Index;
}

Chaos::FStrainedProxyRange::FStrainedProxyRange(Chaos::FRigidClustering& InRigidClustering, const bool bRootLevelOnly, const TArray<FPBDRigidClusteredParticleHandle*>* InStrainedParticles)
	: RigidClustering(InRigidClustering)
	, StrainedParticles(InStrainedParticles)
{
	if (!StrainedParticles)
	{
		return;
	}

	ProxyAndRoots.Reserve(StrainedParticles->Num());
	for (Chaos::FPBDRigidClusteredParticleHandle* Cluster : *StrainedParticles)
	{
		// Make sure the cluster's physics proxy exists and is the right type
		IPhysicsProxyBase* Proxy = Cluster->PhysicsProxy();
		if (Proxy == nullptr)
		{
			return;
		}

		const bool bIsGeometryCollectionProxy = (Proxy->GetType() == EPhysicsProxyType::GeometryCollectionType);
		if (bIsGeometryCollectionProxy)
		{
			FGeometryCollectionPhysicsProxy* ProxyGC = static_cast<FGeometryCollectionPhysicsProxy*>(Proxy);

			// Make sure the rest collection has a root index (If a GC)
			if (bRootLevelOnly)
			{
				FSimulationParameters& Parameters = ProxyGC->GetSimParameters();
				const int32 RootIndex = Parameters.InitialRootIndex;
				Chaos::FPBDRigidClusteredParticleHandle* ParticleHandle = ProxyGC->GetParticle_Internal(RootIndex);
				if (ParticleHandle != nullptr)
				{
					if (Cluster != ParticleHandle)
					{
						return;
					}
				}

				// Only need to use AddUnique if we're not checking for root, since at most
				// one cluster will have the rest collection's root index.
				ProxyAndRoots.Add({ ProxyGC, Cluster, /*bPartialDestruction*/ false });
			}
			else
			{

				ProxyAndRoots.AddUnique({ ProxyGC, Cluster, /*bPartialDestruction*/ false});
			}
		}

		// if the cluster union is the one reported we are in a partial destruction scenario
		const bool bIsClusterUnionProxy = (Proxy->GetType() == EPhysicsProxyType::ClusterUnionProxy);
		if (bIsClusterUnionProxy)
		{
			// for cluster union we go through the children 
			Chaos::FClusterUnionPhysicsProxy* ProxyUC = static_cast<FClusterUnionPhysicsProxy*>(Proxy);
			if (const TArray<Chaos::FPBDRigidParticleHandle*>* ClusterUnionChildren = RigidClustering.GetChildrenMap().Find(ProxyUC->GetParticle_Internal()))
			{
				for (Chaos::FPBDRigidParticleHandle* ChildHandle : *ClusterUnionChildren)
				{
					if (ChildHandle)
					{
						if (Chaos::FPBDRigidClusteredParticleHandle* ClusteredChildHandle = ChildHandle->CastToClustered())
						{
							if (ClusteredChildHandle->GetExternalStrain() > 0 || ClusteredChildHandle->CollisionImpulse() > 0)
							{
								IPhysicsProxyBase* ChildProxy = ClusteredChildHandle->PhysicsProxy();
								if (ChildProxy && ChildProxy->GetType() == EPhysicsProxyType::GeometryCollectionType)
								{
									// we actually add tyhe cluster union proxy and the partial destruction child handle
									FGeometryCollectionPhysicsProxy* ChildProxyGC = static_cast<FGeometryCollectionPhysicsProxy*>(ChildProxy);
									ProxyAndRoots.Add({ ChildProxyGC, ClusteredChildHandle, /*bPartialDestruction*/ true });
								}
							}
						}
					}
				}
			}
		}
	}
}

Chaos::FStrainedProxyRange Chaos::FStrainModifierAccessor::GetStrainedProxies(const bool bRootLevelOnly)
{
	return FStrainedProxyRange(RigidClustering, bRootLevelOnly, StrainedParticles);
}

