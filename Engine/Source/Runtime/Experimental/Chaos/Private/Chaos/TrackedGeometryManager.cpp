// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/TrackedGeometryManager.h"
#include "Chaos/ChaosArchive.h"
#include "Misc/ScopeLock.h"

namespace Chaos
{
	FTrackedGeometryManager& FTrackedGeometryManager::Get()
	{
		static FTrackedGeometryManager Singleton;
		return Singleton;
	}

	void FTrackedGeometryManager::DumpMemoryUsage(FOutputDevice* Ar) const
	{
		const TCHAR* Empty = TEXT("");
		DumpMemoryUsage(Empty, Ar);
	}

	void FTrackedGeometryManager::DumpMemoryUsage(const TCHAR* Cmd, FOutputDevice* Ar) const
	{
		struct FMemInfo
		{
			uint32 NumBytes;
			FString DebugInfo;
			FString ToString;

			bool operator<(const FMemInfo& Other) const { return NumBytes < Other.NumBytes; }
		};

		EImplicitObjectType FilterType = ImplicitObjectType::ConcreteObjectCount;

		if(FParse::Param(Cmd, TEXT("-ListTypes")))
		{
			Ar->Logf(TEXT("Available Geometry Types:"));
			for(uint8 TypeIndex = 0; TypeIndex < ImplicitObjectType::ConcreteObjectCount; ++TypeIndex)
			{
				Ar->Logf(TEXT("\t%s"), ImplicitObjectTypeToString(EImplicitObjectType(TypeIndex)));
			}
			return;
		}

		FString FilterString;
		if(FParse::Value(Cmd, TEXT("TypeFilter="), FilterString))
		{
			FilterType = ImplicitObjectTypeFromString(FilterString);
		}

		TArray<FMemInfo> MemEntries;
		uint32 TotalBytes = 0;
		for(const auto& Itr : SharedGeometry)
		{
			FImplicitObject* NonConst = const_cast<FImplicitObject*>(Itr.Key.Get());	//only doing this to write out, serialize is non const for read in

			if(FilterType != ImplicitObjectType::ConcreteObjectCount)
			{
				if(!(NonConst->GetType() == FilterType || NonConst->GetCollisionType() == FilterType))
				{
					continue;
				}
			}

			FMemInfo Info;
			Info.DebugInfo = Itr.Value;
			TArray<uint8> Data;
			FMemoryWriter MemAr(Data);
			FChaosArchive ChaosAr(MemAr);

			NonConst->Serialize(ChaosAr);

			Info.ToString = NonConst->ToString();
			Info.NumBytes = Data.Num();
			MemEntries.Add(Info);
			TotalBytes += Info.NumBytes;
		}

		MemEntries.Sort();

		Ar->Logf(TEXT(""));
		Ar->Logf(TEXT("Chaos Tracked Geometry:"));
		Ar->Logf(TEXT(""));

		for(const FMemInfo& Info : MemEntries)
		{
			Ar->Logf(TEXT("%-10d %s ToString:%s"), Info.NumBytes, *Info.DebugInfo, *Info.ToString);
		}

		Ar->Logf(TEXT("%-10d Total"), TotalBytes);
	}

	void FTrackedGeometryManager::AddGeometry(TSerializablePtr<FImplicitObject> Geometry, const FString& DebugInfo)
	{
		UE::TScopeLock Lock(CriticalSection);
		SharedGeometry.Add(Geometry, DebugInfo);
	}

	void FTrackedGeometryManager::RemoveGeometry(const FImplicitObject* Geometry)
	{
		UE::TScopeLock Lock(CriticalSection);
		TSerializablePtr<FImplicitObject> Dummy;
		Dummy.SetFromRawLowLevel(Geometry);
		SharedGeometry.Remove(Dummy);
	}

}
