// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/TriangleMesh.h"

#include "Chaos/Box.h"
#include "Chaos/Defines.h"
#include "Chaos/Plane.h"
#include "Chaos/SmoothProject.h"
#include "Chaos/Triangle.h"
#include "Chaos/TriangleCollisionPoint.h"
#include "HAL/IConsoleManager.h"
#include "Math/NumericLimits.h"
#include "Math/RandomStream.h"
#include "Templates/Sorting.h"
#include "Templates/TypeHash.h"
#include "Chaos/HierarchicalSpatialHash.h"
#include "Chaos/PBDFlatWeightMap.h"

#include <algorithm>
#include <iostream>

#if INTEL_ISPC
#include "TriangleMesh.ispc.generated.h"
static_assert(sizeof(ispc::FVector3f) == sizeof(Chaos::TVec3<Chaos::FRealSingle>), "sizeof(ispc::FVector3f) != sizeof(Chaos::TVec3<Chaos::FRealSingle>)");
static_assert(sizeof(ispc::TArrayInt) == sizeof(TArray<int32>), "sizeof(ispc::TArrayInt) != sizeof(TArray<int32>)");
#endif

#if !defined(CHAOS_TRIANGLE_MESH_ISPC_ENABLED_DEFAULT)
#define CHAOS_TRIANGLE_MESH_ISPC_ENABLED_DEFAULT 1
#endif

#if !defined(USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING)
#define USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING 0
#endif

// Support run-time toggling on supported platforms in non-shipping configurations
#if !INTEL_ISPC || (UE_BUILD_SHIPPING && !USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING)
static constexpr bool bChaos_TriangleMesh_ISPC_Enabled = INTEL_ISPC && CHAOS_TRIANGLE_MESH_ISPC_ENABLED_DEFAULT;
#else
static bool bChaos_TriangleMesh_ISPC_Enabled = CHAOS_TRIANGLE_MESH_ISPC_ENABLED_DEFAULT;
static FAutoConsoleVariableRef CVarChaosTriangleMeshISPCEnabled(TEXT("p.Chaos.TriangleMesh.ISPC"), bChaos_TriangleMesh_ISPC_Enabled, TEXT("Whether to use ISPC optimizations in triangle mesh calculations"));
#endif

namespace Chaos
{
template<typename T>
struct TTriangleMeshBvEntry
{
	const FTriangleMesh* TmData;
	const TConstArrayView<TVec3<T>>* Points;
	int32 Index;

	bool HasBoundingBox() const 
	{
		return true;
	}

	Chaos::TAABB<T,3> BoundingBox() const
	{
		const TVec3<int32>& Tri = TmData->GetElements()[Index];
		const TVec3<T>& A = (*Points)[Tri[0]];
		const TVec3<T>& B = (*Points)[Tri[1]];
		const TVec3<T>& C = (*Points)[Tri[2]];

		TAABB<T,3> Bounds(A, A);

		Bounds.GrowToInclude(B);
		Bounds.GrowToInclude(C);

		return Bounds;
	}

	typename FTriangleMesh::TSpatialHashType<T>::FVectorAABB VectorAABB() const
	{
		const TVec3<int32>& Tri = TmData->GetElements()[Index];
		const TVec3<T>& A = (*Points)[Tri[0]];
		const TVec3<T>& B = (*Points)[Tri[1]];
		const TVec3<T>& C = (*Points)[Tri[2]];
		typename FTriangleMesh::TSpatialHashType<T>::FVectorAABB AABB(A);
		AABB.GrowToInclude(B);
		AABB.GrowToInclude(C);
		return AABB;
	}

	template<typename TPayloadType>
	int32 GetPayload(int32 Idx) const { return Idx; }
};

template<typename T>
struct TTriangleMeshBvData
{
	const FTriangleMesh* TmData;
	const TConstArrayView<TVec3<T>>* Points;

	TTriangleMeshBvEntry<T> operator[](const int32 ParticleIndex) const
	{
		return TTriangleMeshBvEntry<T>{ TmData, Points, ParticleIndex };
	}

	int32 Num() const
	{
		return TmData->GetNumElements();
	}
};

struct FTriangleMeshBvDataWithThicknessEntry
{
	const FTriangleMesh* TmData;
	const TConstArrayView<Softs::FSolverVec3>& Points;
	const Softs::FPBDFlatWeightMap& PointThicknesses;
	int32 ThicknessMapIndexOffset;
	int32 Index;

	bool HasBoundingBox() const
	{
		return true;
	}

	typename FTriangleMesh::TSpatialHashType<Softs::FSolverReal>::FVectorAABB VectorAABB() const
	{
		const TVec3<int32>& Tri = TmData->GetElements()[Index];
		const Softs::FSolverVec3& A = Points[Tri[0]];
		const Softs::FSolverVec3& B = Points[Tri[1]];
		const Softs::FSolverVec3& C = Points[Tri[2]];
		typename FTriangleMesh::TSpatialHashType<Softs::FSolverReal>::FVectorAABB AABB(A, PointThicknesses.GetValue(Tri[0] - ThicknessMapIndexOffset));
		AABB.GrowToInclude(B, PointThicknesses.GetValue(Tri[1] - ThicknessMapIndexOffset));
		AABB.GrowToInclude(C, PointThicknesses.GetValue(Tri[2] - ThicknessMapIndexOffset));
		return AABB;
	}

	template<typename TPayloadType>
	int32 GetPayload(int32 Idx) const { return Idx; }
};

struct FTriangleMeshBvDataWithThickness
{
	const FTriangleMesh* TmData;
	const TConstArrayView<Softs::FSolverVec3>& Points;
	const Softs::FPBDFlatWeightMap& PointThicknesses;
	const int32 ThicknessMapIndexOffset;

	FTriangleMeshBvDataWithThicknessEntry operator[](const int32 ParticleIndex) const
	{
		return FTriangleMeshBvDataWithThicknessEntry{ TmData, Points, PointThicknesses, ThicknessMapIndexOffset, ParticleIndex };
	}

	int32 Num() const
	{
		return TmData->GetNumElements();
	}
};

FTriangleMesh::FTriangleMesh()
    : MStartIdx(0)
    , MNumIndices(0)
{}

FTriangleMesh::FTriangleMesh(TArray<TVec3<int32>>&& Elements, const int32 StartIdx, const int32 EndIdx, const bool CullDegenerateElements)
{
	Init(Elements, StartIdx, EndIdx, CullDegenerateElements);
}

FTriangleMesh::FTriangleMesh(FTriangleMesh&& Other)
    : MElements(MoveTemp(Other.MElements))
    , MPointToTriangleMap(MoveTemp(Other.MPointToTriangleMap))
    , MStartIdx(Other.MStartIdx)
    , MNumIndices(Other.MNumIndices)
{}

FTriangleMesh::~FTriangleMesh()
{}

void FTriangleMesh::Init(TArray<TVec3<int32>>&& Elements, const int32 StartIdx, const int32 EndIdx, const bool CullDegenerateElements)
{
	MElements = MoveTemp(Elements);
	MStartIdx = 0;
	MNumIndices = 0;
	ResetAuxiliaryStructures();
	InitHelper(StartIdx, EndIdx, CullDegenerateElements);
}

void FTriangleMesh::Init(const TConstArrayView<TVec3<int32>>& Elements, const int32 StartIdx, const int32 EndIdx, const bool CullDegenerateElements)
{
	MElements = Elements;
	MStartIdx = 0;
	MNumIndices = 0;
	ResetAuxiliaryStructures();
	InitHelper(StartIdx, EndIdx, CullDegenerateElements);
}

void FTriangleMesh::InitHelper(const int32 StartIdx, const int32 EndIdx, const bool CullDegenerateElements)
{
	if (MElements.Num())
	{
		MStartIdx = MElements[MElements.Num()-1][0];
		int32 MaxIdx = MStartIdx;
		for (int i = MElements.Num()-1; i >= 0 ; --i)
		{
			for (int Axis = 0; Axis < 3; ++Axis)
			{
				MStartIdx = FMath::Min(MStartIdx, MElements[i][Axis]);
				MaxIdx = FMath::Max(MaxIdx, MElements[i][Axis]);
			}
			if (CullDegenerateElements)
			{
				if (MElements[i][0] == MElements[i][1] ||
					MElements[i][0] == MElements[i][2] ||
					MElements[i][1] == MElements[i][2])
				{
					// It's possible that the order of the triangles might be important.
					// RemoveAtSwap() changes the order of the array.  I figure that if
					// you're up for CullDegenerateElements, then triangle reordering is
					// fair game.
					MElements.RemoveAtSwap(i);
				}

			}
		}
		// This assumes vertices are contiguous in the vertex buffer. Assumption is held throughout FTriangleMesh
		MNumIndices = MaxIdx - MStartIdx + 1;
	}
	check(MStartIdx >= 0);
	check(MNumIndices >= 0);
	ExpandVertexRange(StartIdx, EndIdx);
}

void FTriangleMesh::ResetAuxiliaryStructures()
{
	MPointToTriangleMap.Reset();
	MPointToNeighborsMap.Reset();
	TArray<TVec2<int32>> EmptyEdges;
	MSegmentMesh.Init(EmptyEdges);
	MFaceToEdges.Reset();
	MEdgeToFaces.Reset();
}

void FTriangleMesh::Serialize(FArchive& Ar)
{
	Ar << MElements;
	Ar << MStartIdx;
	Ar << MNumIndices;
	if (Ar.IsLoading())
	{
		ResetAuxiliaryStructures();
	}
}

TVec2<int32> FTriangleMesh::GetVertexRange() const
{
	return TVec2<int32>(MStartIdx, MStartIdx + MNumIndices - 1);
}

TSet<int32> FTriangleMesh::GetVertices() const
{
	TSet<int32> Vertices;
	GetVertexSet(Vertices);
	return Vertices;
}

void FTriangleMesh::GetVertexSet(TSet<int32>& VertexSet) const
{
	VertexSet.Reset();
	VertexSet.Reserve(MNumIndices);
	for (const TVec3<int32>& Element : MElements)
	{
		VertexSet.Append({Element[0], Element[1], Element[2]});
	}
}

void FTriangleMesh::GetVertexSetAsArray(TArray<int32>& VertexSet) const
{
	TBitArray<> VisitedVertices; // using local index
	VisitedVertices.Init(false, MNumIndices);
	VertexSet.Reserve(MNumIndices);
	for (const TVec3<int32>& Element : MElements)
	{
		for (int32 Corner = 0; Corner < 3; Corner++)
		{
			const int32 VertexIndex = Element[Corner];
			FBitReference VisitedRef = VisitedVertices[VertexIndex-MStartIdx];
			if (VisitedRef == false)
			{
				VisitedRef = true;
				VertexSet.Add(VertexIndex);
			}
		}
	}
}

const TMap<int32, TSet<int32>>& FTriangleMesh::GetPointToNeighborsMap() const
{
	if (MPointToNeighborsMap.Num())
	{
		return MPointToNeighborsMap;
	}
	MPointToNeighborsMap.Reserve(MNumIndices);
	for (int i = 0; i < MElements.Num(); ++i)
	{
		TSet<int32>& Elems0 = MPointToNeighborsMap.FindOrAdd(MElements[i][0]);
		TSet<int32>& Elems1 = MPointToNeighborsMap.FindOrAdd(MElements[i][1]);
		TSet<int32>& Elems2 = MPointToNeighborsMap.FindOrAdd(MElements[i][2]);
		Elems0.Reserve(Elems0.Num() + 2);
		Elems1.Reserve(Elems1.Num() + 2);
		Elems2.Reserve(Elems2.Num() + 2);

		const TVec3<int32>& Tri = MElements[i];
		Elems0.Add(Tri[1]);
		Elems0.Add(Tri[2]);
		Elems1.Add(Tri[0]);
		Elems1.Add(Tri[2]);
		Elems2.Add(Tri[0]);
		Elems2.Add(Tri[1]);
	}
	return MPointToNeighborsMap;
}

TConstArrayView<TArray<int32>> FTriangleMesh::GetPointToTriangleMap() const
{
	if (!MPointToTriangleMap.Num())
	{
		MPointToTriangleMap.AddDefaulted(MNumIndices);
		for (int i = 0; i < MElements.Num(); ++i)
		{
			for (int Axis = 0; Axis < 3; ++Axis)
			{
				MPointToTriangleMap[MElements[i][Axis] - MStartIdx].Add(i);  // Access MPointToTriangleMap with local index
			}
		}
	}
	return TConstArrayView<TArray<int32>>(MPointToTriangleMap.GetData() - MStartIdx, MStartIdx + MNumIndices);  // Return an array view that is using global indexation
}

TArray<TVec2<int32>> FTriangleMesh::GetUniqueAdjacentPoints() const
{
	TArray<TVec2<int32>> BendingConstraints;
	const TArray<TVec4<int32>> BendingElements = GetUniqueAdjacentElements();
	BendingConstraints.Reset(BendingElements.Num());
	for (const TVec4<int32>& Element : BendingElements)
	{
		BendingConstraints.Emplace(Element[2], Element[3]);
	}
	return BendingConstraints;
}

TArray<TVec4<int32>> FTriangleMesh::GetUniqueAdjacentElements() const
{
	// Build a map with a list of opposite points for every edges
	// OppositePoint = { Index, bEdgeFlipped }
	TMap<TVec2<int32> /*Edge*/, TArray<TTuple<int32, bool>> /*OppositePoints*/> EdgeMap;

	auto SortedEdge = [](int32 P0, int32 P1) { return P0 <= P1 ? TVec2<int32>(P0, P1) : TVec2<int32>(P1, P0); };

	for (const TVec3<int32>& Element : MElements)
	{
		EdgeMap.FindOrAdd(SortedEdge(Element[0], Element[1])).AddUnique({ Element[2], Element[0] > Element[1] });
		EdgeMap.FindOrAdd(SortedEdge(Element[1], Element[2])).AddUnique({ Element[0], Element[1] > Element[2] });
		EdgeMap.FindOrAdd(SortedEdge(Element[2], Element[0])).AddUnique({ Element[1], Element[2] > Element[0] });
	}

	auto OrderOppositePoints = [](const TVec2<int32>& Edge, const TTuple<int32, bool>& Opposite0, const TTuple<int32, bool>& Opposite1)->TVec4<int32>
	{
		// Unflipped opposite point before flipped
		if (!Opposite0.Get<1>() && Opposite1.Get<1>())
		{
			return TVec4<int32>(Edge[0], Edge[1], Opposite0.Get<0>(), Opposite1.Get<0>());
		}
		if (Opposite0.Get<1>() && !Opposite1.Get<1>())
		{
			return TVec4<int32>(Edge[0], Edge[1], Opposite1.Get<0>(), Opposite0.Get<0>());
		}
		// Both same flipped, just order lowest to highest
		if (Opposite0.Get<0>() < Opposite1.Get<0>())
		{
			return TVec4<int32>(Edge[0], Edge[1], Opposite0.Get<0>(), Opposite1.Get<0>());
		}

		return TVec4<int32>(Edge[0], Edge[1], Opposite1.Get<0>(), Opposite0.Get<0>());
	};

	// Build constraints
	TArray<TVec4<int32>> BendingConstraints;
	for (const TPair<TVec2<int32>, TArray<TTuple<int32, bool>>>& EdgeOppositePoints : EdgeMap)
	{
		const TVec2<int32>& Edge = EdgeOppositePoints.Key;
		const TArray<TTuple<int32, bool>>& OppositePoints = EdgeOppositePoints.Value;

		for (int32 Index0 = 0; Index0 < OppositePoints.Num(); ++Index0)
		{
			for (int32 Index1 = Index0 + 1; Index1 < OppositePoints.Num(); ++Index1)
			{
				BendingConstraints.Add(OrderOppositePoints(Edge, OppositePoints[Index0], OppositePoints[Index1] ));
			}
		}
	}

	return BendingConstraints;
}

// Note:	This function assumes Counter Clockwise triangle windings in a Left Handed coordinate system
//			If this is not the case the returned face normals may need to be inverted
template <typename T>
void FTriangleMesh::GetFaceNormals(TArray<TVec3<T>>& Normals, const TConstArrayView<TVec3<T>>& Points, const bool ReturnEmptyOnError) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTriangleMesh_GetFaceNormals);
	Normals.Reset(MElements.Num());
	if (ReturnEmptyOnError)
	{
		for (const TVec3<int32>& Tri : MElements)
		{
			const TVec3<T> p10 = Points[Tri[1]] - Points[Tri[0]];
			const TVec3<T> p20 = Points[Tri[2]] - Points[Tri[0]];
			const TVec3<T> Cross = TVec3<T>::CrossProduct(p20, p10);
			const T Size2 = Cross.SizeSquared();
			if (Size2 < UE_SMALL_NUMBER)
			{
				//particles should not be coincident by the time they get here. Return empty to signal problem to caller
				ensure(false);
				Normals.Empty();
				return;
			}
			else
			{
				Normals.Add(Cross.GetUnsafeNormal());
			}
		}
	}
	else
	{
#if INTEL_ISPC
		if (bChaos_TriangleMesh_ISPC_Enabled && std::is_same_v<T, FRealSingle>)
		{
			Normals.SetNumUninitialized(MElements.Num());
			ispc::GetFaceNormals(
				(ispc::FVector3f*)Normals.GetData(),
				(ispc::FVector3f*)Points.GetData(),
				(ispc::FIntVector*)MElements.GetData(),
				MElements.Num());
		}
		else
#endif
		{
			for (const TVec3<int32>& Tri : MElements)
			{
				const TVec3<FRealSingle> p10 = Points[Tri[1]] - Points[Tri[0]];
				const TVec3<FRealSingle> p20 = Points[Tri[2]] - Points[Tri[0]];
				const TVec3<FRealSingle> Cross = TVec3<FRealSingle>::CrossProduct(p20, p10);
				Normals.Add(Cross.GetSafeNormal());
			}
		}
	}
}
template CHAOS_API void FTriangleMesh::GetFaceNormals<FRealSingle>(TArray<TVec3<FRealSingle>>&, const TConstArrayView<TVec3<FRealSingle>>&, const bool) const;
template CHAOS_API void FTriangleMesh::GetFaceNormals<FRealDouble>(TArray<TVec3<FRealDouble>>&, const TConstArrayView<TVec3<FRealDouble>>&, const bool) const;

template <typename T>
TArray<TVec3<T>> FTriangleMesh::GetFaceNormals(const TConstArrayView<TVec3<T>>& Points, const bool ReturnEmptyOnError) const
{
	TArray<TVec3<T>> Normals;
	GetFaceNormals(Normals, Points, ReturnEmptyOnError);
	return Normals;
}
template CHAOS_API TArray<TVec3<FRealSingle>> FTriangleMesh::GetFaceNormals<FRealSingle>(const TConstArrayView<TVec3<FRealSingle>>&, const bool) const;
template CHAOS_API TArray<TVec3<FRealDouble>> FTriangleMesh::GetFaceNormals<FRealDouble>(const TConstArrayView<TVec3<FRealDouble>>&, const bool) const;

TArray<FVec3> FTriangleMesh::GetPointNormals(const TConstArrayView<FVec3>& Points, const bool ReturnEmptyOnError, const bool bUseGlobalArray)
{
	TArray<FVec3> PointNormals;
	const TArray<FVec3> FaceNormals = GetFaceNormals(Points, ReturnEmptyOnError);
	if (FaceNormals.Num())
	{
		PointNormals.SetNumUninitialized(bUseGlobalArray ? MNumIndices+MStartIdx : MNumIndices);
		GetPointNormals(PointNormals, FaceNormals, bUseGlobalArray);
	}
	return PointNormals;
}

void FTriangleMesh::GetPointNormals(TArrayView<FVec3> PointNormals, const TConstArrayView<FVec3>& FaceNormals, const bool bUseGlobalArray)
{
	GetPointToTriangleMap(); // build MPointToTriangleMap
	const FTriangleMesh* ConstThis = this;
	ConstThis->GetPointNormals(PointNormals, FaceNormals, bUseGlobalArray);
}

template <typename T>
void FTriangleMesh::GetPointNormals(TArrayView<TVec3<T>> PointNormals, const TConstArrayView<TVec3<T>>& FaceNormals, const bool bUseGlobalArray) const
{
	check(MPointToTriangleMap.Num() != 0);
	TRACE_CPUPROFILER_EVENT_SCOPE(FTriangleMesh_GetPointNormals);
	for (int32 Element = 0; Element < MNumIndices; ++Element)  // Iterate points with local indexes
	{
		const int32 NormalIndex = bUseGlobalArray ? LocalToGlobal(Element) : Element;  // Select whether the points normal indices match the points indices or start at 0
		TVec3<T> Normal((T)0);
		const TArray<int32>& TriangleMap = MPointToTriangleMap[Element];  // Access MPointToTriangleMap with local index
		for (int32 k = 0; k < TriangleMap.Num(); ++k)
		{
			if (FaceNormals.IsValidIndex(TriangleMap[k]))
			{
				Normal += FaceNormals[TriangleMap[k]];
			}
		}
		PointNormals[NormalIndex] = Normal.GetSafeNormal();
	}
}
template CHAOS_API void FTriangleMesh::GetPointNormals<FRealDouble>(TArrayView<TVec3<FRealDouble>>, const TConstArrayView<TVec3<FRealDouble>>&, const bool) const;

CHAOS_API void FTriangleMesh::GetPointNormals(TArrayView<TVec3<FRealSingle>> PointNormals, const TConstArrayView<TVec3<FRealSingle>>& FaceNormals, const bool bUseGlobalArray) const
{
	check(MPointToTriangleMap.Num() != 0);

#if INTEL_ISPC
	if (bChaos_TriangleMesh_ISPC_Enabled)
	{
		ispc::GetPointNormals(
			(ispc::FVector3f*)PointNormals.GetData(),
			(const ispc::FVector3f*)FaceNormals.GetData(),
			(const ispc::TArrayInt*)MPointToTriangleMap.GetData(),
			bUseGlobalArray ? LocalToGlobal(0) : 0,
			FaceNormals.Num(),
			MNumIndices);
	}
	else
#endif
	{
		for (int32 Element = 0; Element < MNumIndices; ++Element)  // Iterate points with local indexes
		{
			const int32 NormalIndex = bUseGlobalArray ? LocalToGlobal(Element) : Element;  // Select whether the points normal indices match the points indices or start at 0
			TVec3<FRealSingle> Normal(0.f);
			const TArray<int32>& TriangleMap = MPointToTriangleMap[Element];  // Access MPointToTriangleMap with local index
			for (int32 k = 0; k < TriangleMap.Num(); ++k)
			{
				if (FaceNormals.IsValidIndex(TriangleMap[k]))
				{
					Normal += FaceNormals[TriangleMap[k]];
				}
			}
			PointNormals[NormalIndex] = Normal.GetSafeNormal();
		}
	}
}

template<>
CHAOS_API void FTriangleMesh::GetPointNormals<FRealSingle>(TArrayView<TVec3<FRealSingle>> PointNormals, const TConstArrayView<TVec3<FRealSingle>>& FaceNormals, const bool bUseGlobalArray) const
{
	return GetPointNormals(PointNormals, FaceNormals, bUseGlobalArray);
}

template<class T>
void AddTrianglesToHull(const TConstArrayView<FVec3>& Points, const int32 I0, const int32 I1, const int32 I2, const TPlane<T, 3>& SplitPlane, const TArray<int32>& InIndices, TArray<TVec3<int32>>& OutIndices)
{
	int32 MaxD = 0; //This doesn't need to be initialized but we need to avoid the compiler warning
	T MaxDistance = 0;
	for (int32 i = 0; i < InIndices.Num(); ++i)
	{
		T Distance = SplitPlane.SignedDistance(Points[InIndices[i]]);
		check(Distance >= 0);
		if (Distance > MaxDistance)
		{
			MaxDistance = Distance;
			MaxD = InIndices[i];
		}
	}
	if (MaxDistance == 0)
	{
		//@todo(mlentine): Do we need to do anything here when InIndices is > 0?
		check(I0 != I1);
		check(I1 != I2);
		OutIndices.AddUnique(TVec3<int32>(I0, I1, I2));
		return;
	}
	if (MaxDistance > 0)
	{
		const FVec3& NewX = Points[MaxD];
		const FVec3& X0 = Points[I0];
		const FVec3& X1 = Points[I1];
		const FVec3& X2 = Points[I2];
		const FVec3 V1 = (NewX - X0).GetSafeNormal();
		const FVec3 V2 = (NewX - X1).GetSafeNormal();
		const FVec3 V3 = (NewX - X2).GetSafeNormal();
		FVec3 Normal1 = FVec3::CrossProduct(V1, V2).GetSafeNormal();
		if (FVec3::DotProduct(Normal1, X2 - X0) > 0)
		{
			Normal1 *= -1.0f;
		}
		FVec3 Normal2 = FVec3::CrossProduct(V1, V3).GetSafeNormal();
		if (FVec3::DotProduct(Normal2, X1 - X0) > 0)
		{
			Normal2 *= -1.0f;
		}
		FVec3 Normal3 = FVec3::CrossProduct(V2, V3).GetSafeNormal();
		if (FVec3::DotProduct(Normal3, X0 - X1) > 0)
		{
			Normal3 *= -1.0f;
		}
		TPlane<FReal, 3> NewPlane1(NewX, Normal1);
		TPlane<FReal, 3> NewPlane2(NewX, Normal2);
		TPlane<FReal, 3> NewPlane3(NewX, Normal3);
		TArray<int32> NewIndices1;
		TArray<int32> NewIndices2;
		TArray<int32> NewIndices3;
		TSet<FIntVector> FacesToFilter;
		for (int32 i = 0; i < InIndices.Num(); ++i)
		{
			if (MaxD == InIndices[i])
			{
				continue;
			}
			T Dist1 = NewPlane1.SignedDistance(Points[InIndices[i]]);
			T Dist2 = NewPlane2.SignedDistance(Points[InIndices[i]]);
			T Dist3 = NewPlane3.SignedDistance(Points[InIndices[i]]);
			check(Dist1 < 0 || Dist2 < 0 || Dist3 < 0);
			if (Dist1 > 0 && Dist2 > 0)
			{
				FacesToFilter.Add(FIntVector(I0, MaxD, InIndices[i]));
			}
			if (Dist1 > 0 && Dist3 > 0)
			{
				FacesToFilter.Add(FIntVector(I1, MaxD, InIndices[i]));
			}
			if (Dist2 > 0 && Dist3 > 0)
			{
				FacesToFilter.Add(FIntVector(I2, MaxD, InIndices[i]));
			}
			if (Dist1 >= 0)
			{
				NewIndices1.Add(InIndices[i]);
			}
			if (Dist2 >= 0)
			{
				NewIndices2.Add(InIndices[i]);
			}
			if (Dist3 >= 0)
			{
				NewIndices3.Add(InIndices[i]);
			}
		}
		AddTrianglesToHull(Points, I0, I1, MaxD, NewPlane1, NewIndices1, OutIndices);
		AddTrianglesToHull(Points, I0, I2, MaxD, NewPlane2, NewIndices2, OutIndices);
		AddTrianglesToHull(Points, I1, I2, MaxD, NewPlane3, NewIndices3, OutIndices);
		for (int32 i = 0; i < OutIndices.Num(); ++i)
		{
			if (FacesToFilter.Contains(FIntVector(OutIndices[i][0], OutIndices[i][1], OutIndices[i][2])))
			{
				OutIndices.RemoveAtSwap(i);
				i--;
			}
		}
	}
}

// @todo(mlentine, ocohen); Merge different hull creation versions
FTriangleMesh FTriangleMesh::GetConvexHullFromParticles(const TConstArrayView<FVec3>& Points)
{
	TArray<TVec3<int32>> Indices;
	if (Points.Num() <= 2)
	{
		return FTriangleMesh(MoveTemp(Indices));
	}
	// Find max and min x points
	int32 MinX = 0;
	int32 MaxX = 0;
	int32 MinY = 0;
	int32 MaxY = 0;
	int32 Index1 = 0;
	int32 Index2 = 0;
	for (int32 Idx = 1; Idx < Points.Num(); ++Idx)
	{
		if (Points[Idx][0] > Points[MaxX][0])
		{
			MaxX = Idx;
		}
		if (Points[Idx][0] < Points[MinX][0])
		{
			MinX = Idx;
		}
		if (Points[Idx][1] > Points[MaxY][1])
		{
			MaxY = Idx;
		}
		if (Points[Idx][1] < Points[MinY][1])
		{
			MinY = Idx;
		}
	}
	if (MaxX == MinX && MinY == MaxY && MinX == MinY)
	{
		// Points are co-linear
		return FTriangleMesh(MoveTemp(Indices));
	}
	// Find max distance
	FReal DistanceY = (Points[MaxY] - Points[MinY]).Size();
	FReal DistanceX = (Points[MaxX] - Points[MinX]).Size();
	if (DistanceX > DistanceY)
	{
		Index1 = MaxX;
		Index2 = MinX;
	}
	else
	{
		Index1 = MaxY;
		Index2 = MinY;
	}
	const FVec3& X1 = Points[Index1];
	const FVec3& X2 = Points[Index2];
	FReal MaxDist = 0;
	int32 MaxD = -1;
	for (int32 Idx = 0; Idx < Points.Num(); ++Idx)
	{
		if (Idx == Index1 || Idx == Index2)
		{
			continue;
		}
		const FVec3& X0 = Points[Idx];
		FReal Distance = FVec3::CrossProduct(X0 - X1, X0 - X2).Size() / (X2 - X1).Size();
		if (Distance > MaxDist)
		{
			MaxDist = Distance;
			MaxD = Idx;
		}
	}
	if (MaxD != -1)
	{
		const FVec3& X0 = Points[MaxD];
		FVec3 Normal = FVec3::CrossProduct((X0 - X1).GetSafeNormal(), (X0 - X2).GetSafeNormal());
		TPlane<FReal, 3> SplitPlane(X0, Normal);
		TPlane<FReal, 3> SplitPlaneNeg(X0, -Normal);
		TArray<int32> Left;
		TArray<int32> Right;
		TArray<int32> Coplanar;
		TSet<int32> CoplanarSet;
		CoplanarSet.Add(MaxD);
		CoplanarSet.Add(Index1);
		CoplanarSet.Add(Index2);
		for (int32 Idx = 0; Idx < Points.Num(); ++Idx)
		{
			if (Idx == Index1 || Idx == Index2 || Idx == MaxD)
			{
				continue;
			}
			if (SplitPlane.SignedDistance(Points[Idx]) > 0)
			{
				Left.Add(Idx);
			}
			else if (SplitPlane.SignedDistance(Points[Idx]) < 0)
			{
				Right.Add(Idx);
			}
			else
			{
				CoplanarSet.Add(Idx);
				Coplanar.Add(Idx);
			}
		}
		if (!Left.Num())
		{
			Right.Append(Coplanar);
			AddTrianglesToHull(Points, MaxD, Index1, Index2, SplitPlane, Left, Indices);
			AddTrianglesToHull(Points, MaxD, Index1, Index2, SplitPlaneNeg, Right, Indices);
		}
		else if (!Right.Num())
		{
			Left.Append(Coplanar);
			AddTrianglesToHull(Points, MaxD, Index1, Index2, SplitPlane, Left, Indices);
			AddTrianglesToHull(Points, MaxD, Index1, Index2, SplitPlaneNeg, Right, Indices);
		}
		else if (Left.Num() && Right.Num())
		{
			Right.Append(Coplanar);
			Left.Append(Coplanar);
			AddTrianglesToHull(Points, MaxD, Index1, Index2, SplitPlane, Left, Indices);
			AddTrianglesToHull(Points, MaxD, Index1, Index2, SplitPlaneNeg, Right, Indices);
			// Remove combinations of MaxD, Index1, Index2, and Coplanar
			for (int32 i = 0; i < Indices.Num(); ++i)
			{
				if (CoplanarSet.Contains(Indices[i].X) && CoplanarSet.Contains(Indices[i].Y) && CoplanarSet.Contains(Indices[i].Z))
				{
					Indices.RemoveAtSwap(i);
					i--;
				}
			}
		}
	}
	return FTriangleMesh(MoveTemp(Indices));
}

FORCEINLINE TVec2<int32> GetOrdered(const TVec2<int32>& elem)
{
	const bool ordered = elem[0] < elem[1];
	return TVec2<int32>(
	    ordered ? elem[0] : elem[1],
	    ordered ? elem[1] : elem[0]);
}

void Order(int32& A, int32& B)
{
	if (B < A)
	{
		int32 Tmp = A;
		A = B;
		B = Tmp;
	}
}

TVec3<int32> GetOrdered(const TVec3<int32>& Elem)
{
	TVec3<int32> OrderedElem = Elem;	   // 3 2 1		1 2 3		1 2 1	2 1 1
	Order(OrderedElem[0], OrderedElem[1]); // 2 3 1		1 2 3		1 2 1	1 2 1
	Order(OrderedElem[1], OrderedElem[2]); // 2 1 3		1 2 3		1 1 2	1 1 2
	Order(OrderedElem[0], OrderedElem[1]); // 1 2 3		1 2 3		1 1 2	1 1 2
	return OrderedElem;
}

/**
 * Comparator for TSet<TVec2<int32>> that compares the components of vectors in ascending
 * order.
 */
struct OrderedEdgeKeyFuncs : BaseKeyFuncs<TVec2<int32>, TVec2<int32>, false>
{
	static FORCEINLINE TVec2<int32> GetSetKey(const TVec2<int32>& elem)
	{
		return GetOrdered(elem);
	}

	static FORCEINLINE bool Matches(const TVec2<int32>& a, const TVec2<int32>& b)
	{
		const auto orderedA = GetSetKey(a);
		const auto orderedB = GetSetKey(b);
		return orderedA[0] == orderedB[0] && orderedA[1] == orderedB[1];
	}

	static FORCEINLINE uint32 GetKeyHash(const TVec2<int32>& elem)
	{
		const uint32 v = HashCombine(::GetTypeHash(elem[0]), ::GetTypeHash(elem[1]));
		return v;
	}
};

const FSegmentMesh& FTriangleMesh::GetSegmentMesh() const
{
	if (MSegmentMesh.GetNumElements() != 0)
	{
		return MSegmentMesh;
	}

	// Array of ordered edges; other mappings will refer to edges via their index in this array
	TArray<TVec2<int32>> UniqueEdges;
	UniqueEdges.Reserve(MElements.Num() * 3);
	// Map to accelerate checking if an edge is already in the UniqueEdges array
	TMap<TVector<int32, 2>, int32> EdgeToIdx;
	EdgeToIdx.Reserve(MElements.Num() * 3);

	MEdgeToFaces.Reset();
	MEdgeToFaces.Reserve(MElements.Num() * 3); // over estimate
	MFaceToEdges.Reset();
	MFaceToEdges.SetNum(MElements.Num());
	for (int32 FaceIdx = 0; FaceIdx < MElements.Num(); FaceIdx++)
	{
		const TVec3<int32>& Tri = MElements[FaceIdx];
		TVec3<int32>& EdgeIds = MFaceToEdges[FaceIdx];
		for (int32 j = 0; j < 3; j++)
		{
			TVec2<int32> Edge(Tri[j], Tri[(j + 1) % 3]);

			int32 EdgeIdx;
			TVector<int32, 2> OrderedEdge = GetOrdered(Edge);
			int32* FoundEdgeIdx = EdgeToIdx.Find(OrderedEdge);
			if (!FoundEdgeIdx)
			{
				EdgeIdx = UniqueEdges.Add(OrderedEdge);
				EdgeToIdx.Add(OrderedEdge, EdgeIdx);
			}
			else
			{
				EdgeIdx = *FoundEdgeIdx;
			}
			EdgeIds[j] = EdgeIdx;

			// Track which faces are shared by edges.
			const int currNum = MEdgeToFaces.Num();
			if (currNum <= EdgeIdx)
			{
				// Add and initialize new entries
				MEdgeToFaces.SetNum(EdgeIdx + 1, EAllowShrinking::No);
				for (int32 k = currNum; k < EdgeIdx + 1; k++)
				{
					MEdgeToFaces[k] = TVec2<int32>(-1, -1);
				}
			}

			TVec2<int32>& FacesSharingThisEdge = MEdgeToFaces[EdgeIdx];
			if (FacesSharingThisEdge[0] < 0)
			{
				// 0th initialized, but not set
				FacesSharingThisEdge[0] = FaceIdx;
			}
			else if (FacesSharingThisEdge[1] < 0)
			{
				// 0th already set, only 1 is left
				FacesSharingThisEdge[1] = FaceIdx;
			}
			else
			{
				// This is a non-manifold mesh, where this edge is shared by
				// more than 2 faces.
				CHAOS_ENSURE_MSG(false, TEXT("Skipping non-manifold edge to face mapping."));
			}
		}
	}
	MSegmentMesh.Init(MoveTemp(UniqueEdges));
	return MSegmentMesh;
}

const TArray<TVec3<int32>>& FTriangleMesh::GetFaceToEdges() const
{
	GetSegmentMesh();
	return MFaceToEdges;
}

const TArray<TVec2<int32>>& FTriangleMesh::GetEdgeToFaces() const
{
	GetSegmentMesh();
	return MEdgeToFaces;
}


TSet<int32> FTriangleMesh::GetBoundaryPoints()
{
	const FSegmentMesh& SegmentMesh = GetSegmentMesh();
	const TArray<TVec2<int32>>& Edges = SegmentMesh.GetElements();
	const TArray<TVec2<int32>>& EdgeToFaces = GetEdgeToFaces();
	TSet<int32> OpenBoundaryPoints;
	for (int32 EdgeIdx = 0; EdgeIdx < EdgeToFaces.Num(); ++EdgeIdx)
	{
		const TVec2<int32>& CoincidentFaces = EdgeToFaces[EdgeIdx];
		if (CoincidentFaces[0] == INDEX_NONE || CoincidentFaces[1] == INDEX_NONE)
		{
			const TVec2<int32>& Edge = Edges[EdgeIdx];
			OpenBoundaryPoints.Add(Edge[0]);
			OpenBoundaryPoints.Add(Edge[1]);
		}
	}
	return OpenBoundaryPoints;
}

TMap<int32, int32> FTriangleMesh::FindCoincidentVertexRemappings(
	const TArray<int32>& TestIndices,
	const TConstArrayView<FVec3>& Points)
{
	// From index -> To index
	TMap<int32, int32> Remappings;

	const int32 NumPoints = TestIndices.Num();
	if (NumPoints <= 1)
	{
		return Remappings;
	}

	// Move the points to the origin to avoid floating point aliasing far away
	// from the origin.
	FAABB3 Bbox(Points[TestIndices[0]], Points[TestIndices[0]]);
	for (int i = 1; i < NumPoints; i++)
	{
		Bbox.GrowToInclude(Points[TestIndices[i]]);
	}
	const FVec3 Center = Bbox.Center();

	TArray<FVec3> LocalPoints;
	LocalPoints.AddUninitialized(NumPoints);
	LocalPoints[0] = Points[TestIndices[0]] - Center;
	FAABB3 LocalBBox(LocalPoints[0], LocalPoints[0]);
	for (int i = 1; i < NumPoints; i++)
	{
		LocalPoints[i] = Points[TestIndices[i]] - Center;
		LocalBBox.GrowToInclude(LocalPoints[i]);
	}

	// Return early if all points are coincident
	if (LocalBBox.Extents().Max() < UE_KINDA_SMALL_NUMBER)
	{
		int32 First = INDEX_NONE;
		for (const int32 Pt : TestIndices)
		{
			if (First == INDEX_NONE)
			{
				First = Pt;
			}
			else
			{
				// Remap Pt to First
				Remappings.Add(Pt, First);
			}
		}
		return Remappings;
	}

	LocalBBox.Thicken(1.0e-3f);
	const FVec3 LocalCenter = LocalBBox.Center();
	const FVec3& LocalMin = LocalBBox.Min();

	const FReal MaxBBoxDim = LocalBBox.Extents().Max();

	// Find coincident vertices.
	// We hash to a grid of fine enough resolution such that if 2 particles 
	// hash to the same cell, then we're going to consider them coincident.
	TMap<int64, TSet<int32>> OccupiedCells;
	OccupiedCells.Reserve(NumPoints);

	const int64 Resolution = static_cast<int64>(floor(MaxBBoxDim / 0.01f));
	const FReal CellSize = static_cast<FReal>(static_cast<double>(MaxBBoxDim) / static_cast<double>(Resolution));
	for (int i = 0; i < 2; i++)
	{
		OccupiedCells.Reset();

		// Shift the grid by 1/2 a grid cell the second iteration so that
		// we don't miss slightly adjacent coincident points across cell
		// boundaries.
		const FVec3 GridCenter = LocalCenter - FVec3(static_cast<FReal>(i) * CellSize / 2.0f);
		for (int32 LocalIdx = 0; LocalIdx < NumPoints; LocalIdx++)
		{
			const int32 Idx = TestIndices[LocalIdx];
			if (i != 0 && Remappings.Contains(Idx))
			{
				// Already remapped
				continue;
			}

			const FVec3& Pos = LocalPoints[LocalIdx];
			const TVec3<int64> Coord(
				static_cast<int64>(FMath::Floor((Pos[0] - GridCenter[0]) / CellSize + static_cast<double>(Resolution) / 2.0f)),
				static_cast<int64>(FMath::Floor((Pos[1] - GridCenter[1]) / CellSize + static_cast<double>(Resolution) / 2.0f)),
				static_cast<int64>(FMath::Floor((Pos[2] - GridCenter[2]) / CellSize + static_cast<double>(Resolution) / 2.0f)));
			const int64 FlatIdx =
				((Coord[0] * Resolution + Coord[1]) * Resolution) + Coord[2];

			TSet<int32>& Bucket = OccupiedCells.FindOrAdd(FlatIdx);
			Bucket.Add(Idx);
		}

		// Iterate over all occupied cells and remap redundant vertices to the first index.
		for (auto& KV : OccupiedCells)
		{
			const TSet<int32>& CoincidentVertices = KV.Value;
			if (CoincidentVertices.Num() <= 1)
				continue;
			int32 First = INDEX_NONE;
			for (const int32 Idx : CoincidentVertices)
			{
				if (First == INDEX_NONE)
				{
					First = Idx;
				}
				else
				{
					Remappings.Add(Idx, First);
				}
			}
		}
	}

	return Remappings;
}

TArray<FReal> FTriangleMesh::GetCurvatureOnEdges(const TArray<FVec3>& FaceNormals)
{
	const int32 NumNormals = FaceNormals.Num();
	check(NumNormals == MElements.Num());
	const FSegmentMesh& SegmentMesh = GetSegmentMesh(); // builds MEdgeToFaces
	TArray<FReal> EdgeAngles;
	EdgeAngles.SetNumZeroed(MEdgeToFaces.Num());
	for (int32 EdgeId = 0; EdgeId < MEdgeToFaces.Num(); EdgeId++)
	{
		const TVec2<int32>& FaceIds = MEdgeToFaces[EdgeId];
		if (FaceIds[0] >= 0 &&
		    FaceIds[1] >= 0 && // -1 is sentinel, which denotes a boundary edge.
		    FaceIds[0] < NumNormals &&
		    FaceIds[1] < NumNormals) // Stay in bounds
		{
			const FVec3& Norm1 = FaceNormals[FaceIds[0]];
			const FVec3& Norm2 = FaceNormals[FaceIds[1]];
			EdgeAngles[EdgeId] = FVec3::AngleBetween(Norm1, Norm2);
		}
	}
	return EdgeAngles;
}

TArray<FReal> FTriangleMesh::GetCurvatureOnEdges(const TConstArrayView<FVec3>& Points)
{
	const TArray<FVec3> FaceNormals = GetFaceNormals(Points, false);
	return GetCurvatureOnEdges(FaceNormals);
}

TArray<FReal> FTriangleMesh::GetCurvatureOnPoints(const TArray<FReal>& EdgeCurvatures)
{
	const FSegmentMesh& SegmentMesh = GetSegmentMesh();
	const TArray<TVec2<int32>>& Segments = SegmentMesh.GetElements();
	check(EdgeCurvatures.Num() == Segments.Num());

	if (MNumIndices < 1)
	{
		return TArray<FReal>();
	}

	TArray<FReal> PointCurvatures;
	// 0.0 means the faces are coplanar.
	// M_PI are as creased as they can be.
	// Initialize to -FLT_MAX so that free particles are penalized.
	PointCurvatures.Init(-TNumericLimits<FReal>::Max(), MNumIndices);
	for (int32 i = 0; i < Segments.Num(); i++)
	{
		const FReal EdgeCurvature = EdgeCurvatures[i];
		const TVec2<int32>& Edge = Segments[i];
		PointCurvatures[GlobalToLocal(Edge[0])] = FMath::Max(PointCurvatures[GlobalToLocal(Edge[0])], EdgeCurvature);
		PointCurvatures[GlobalToLocal(Edge[1])] = FMath::Max(PointCurvatures[GlobalToLocal(Edge[1])], EdgeCurvature);
	}
	return PointCurvatures;
}

TArray<FReal> FTriangleMesh::GetCurvatureOnPoints(const TConstArrayView<FVec3>& Points)
{
	const TArray<FReal> EdgeCurvatures = GetCurvatureOnEdges(Points);
	return GetCurvatureOnPoints(EdgeCurvatures);
}

/**
* Binary predicate for sorting indices according to a secondary array of values to sort
* by.  Puts values into ascending order.
*/
template<class T>
class AscendingPredicate
{
public:
	AscendingPredicate(const TArray<T>& InCompValues, const int32 InOffset)
		: CompValues(InCompValues)
		, Offset(InOffset)
	{}

	bool
		operator()(const int i, const int j) const
	{
		// If an index is out of range, put it at the end.
		const int iOffset = i - Offset;
		const int jOffset = j - Offset;
		const T vi = iOffset >= 0 && iOffset < CompValues.Num() ? CompValues[iOffset] : TNumericLimits<T>::Max();
		const T vj = jOffset >= 0 && jOffset < CompValues.Num() ? CompValues[jOffset] : TNumericLimits<T>::Max();
		return vi < vj;
	}

private:
	const TArray<T>& CompValues;
	const int32 Offset;
};

/**
* Binary predicate for sorting indices according to a secondary array of values to sort
* by.  Puts values into descending order.
*/
template<class T>
class DescendingPredicate
{
public:
	DescendingPredicate(const TArray<T>& CompValues, const int32 Offset = 0)
		: CompValues(CompValues)
		, Offset(Offset)
	{}

	bool
		operator()(const int i, const int j) const
	{
		// If an index is out of range, put it at the end.
		const int iOffset = i - Offset;
		const int jOffset = j - Offset;
		const T vi = iOffset >= 0 && iOffset < CompValues.Num() ? CompValues[iOffset] : -TNumericLimits<T>::Max();
		const T vj = jOffset >= 0 && jOffset < CompValues.Num() ? CompValues[jOffset] : -TNumericLimits<T>::Max();
		return vi > vj;
	}

private:
	const TArray<T>& CompValues;
	const int32 Offset;
};

TArray<int32> FTriangleMesh::GetVertexImportanceOrdering(
    const TConstArrayView<FVec3>& Points,
    const TArray<FReal>& PointCurvatures,
    TArray<int32>* CoincidentVertices,
    const bool RestrictToLocalIndexRange)
{
	const int32 NumPoints = RestrictToLocalIndexRange ? MNumIndices : Points.Num();
	const int32 Offset = RestrictToLocalIndexRange ? MStartIdx : 0;

	TArray<int32> PointOrder;
	if (!NumPoints)
	{
		return PointOrder;
	}

	// Initialize pointOrder to be 0, 1, 2, 3, ..., n-1.
	PointOrder.SetNumUninitialized(NumPoints);
	for (int32 i = 0; i < NumPoints; i++)
	{
		PointOrder[i] = i + Offset;
	}

	if (NumPoints == 1)
	{
		return PointOrder;
	}

	// Move the points to the origin to avoid floating point aliasing far away
	// from the origin.
	FAABB3 Bbox(Points[Offset], Points[Offset]);
	for (int i = 1; i < NumPoints; i++)
	{
		Bbox.GrowToInclude(Points[i + Offset]);
	}
	const FVec3 Center = Bbox.Center();

	TArray<FVec3> LocalPoints;
	LocalPoints.AddUninitialized(NumPoints);
	LocalPoints[0] = Points[Offset] - Center;
	FAABB3 LocalBBox(LocalPoints[0], LocalPoints[0]);
	for (int i = 1; i < NumPoints; i++)
	{
		LocalPoints[i] = Points[Offset + i] - Center;
		LocalBBox.GrowToInclude(LocalPoints[i]);
	}
	LocalBBox.Thicken(1.0e-3f);
	// Center of the local bounding box (should always be close to zero)
	const FVec3 LocalCenter = LocalBBox.Center();
	const FVec3& LocalMin = LocalBBox.Min();

	auto ToFlatIdx = [&LocalPoints, Offset](int32 PointIdx, FVec3 Center, FReal CellSizeIn, int64 Res) -> int64
	{
		const FVec3& Pos = LocalPoints[PointIdx - Offset];
		// grid center co-located at bbox center:
		const TVec3<int64> Coord(
			static_cast<int64>(FMath::Floor((Pos[0] - Center[0]) / CellSizeIn)) + Res / 2,
			static_cast<int64>(FMath::Floor((Pos[1] - Center[1]) / CellSizeIn)) + Res / 2,
			static_cast<int64>(FMath::Floor((Pos[2] - Center[2]) / CellSizeIn)) + Res / 2);
		return ((Coord[0] * Res + Coord[1]) * Res) + Coord[2];
	};

	// Bias towards points further away from the center of the bounding box.
	// Send points that are the furthest away to the front of the list.
	TArray<FReal> Dist;
	Dist.AddUninitialized(NumPoints);
	for (int i = 0; i < NumPoints; i++)
	{
		Dist[i] = (LocalPoints[i] - LocalCenter).SizeSquared();
	}

	// If all points are coincident, return early.
	const FReal MaxBBoxDim = LocalBBox.Extents().Max();
	if (MaxBBoxDim <= 1.0e-6)
	{
		if (CoincidentVertices && NumPoints > 0)
		{
			CoincidentVertices->Append(&PointOrder[1], NumPoints - 1);
		}
		return PointOrder;
	}

	// We've got our base ordering.  Find coincident vertices and send them to
	// the back of the list.  We hash to a grid of fine enough resolution such
	// that if 2 particles hash to the same cell, then we're going to consider
	// them coincident.
	TSet<int64> OccupiedCells;
	OccupiedCells.Reserve(NumPoints);
	if (CoincidentVertices)
	{
		CoincidentVertices->Reserve(64); // a guess
	}
	int32 NumCoincident = 0;
	TArray<uint8> Rank;
	AscendingPredicate<uint8> AscendingRankPred(Rank, Offset); // low to high
	{
		const int64 Resolution = static_cast<int64>(floor(MaxBBoxDim / 0.01));
		const FReal CellSize = static_cast<FReal>(static_cast<double>(MaxBBoxDim) / static_cast<double>(Resolution));
		for (int i = 0; i < 2; i++)
		{
			OccupiedCells.Reset();
			Rank.Reset();
			Rank.AddZeroed(NumPoints);
			// Shift the grid by 1/2 a grid cell the second iteration so that
			// we don't miss slightly adjacent coincident points across cell
			// boundaries.
			// (Note this could still miss some across-cell coincident points, but that's ok because 
			//  the coincident vert array is just used to reduce the number of points used for collisions)
			const FVec3 GridCenter = LocalCenter - FVec3(static_cast<FReal>(i) * CellSize / 2);
			const int NumCoincidentPrev = NumCoincident;
			for (int j = 0; j < NumPoints - NumCoincidentPrev; j++)
			{
				const int32 Idx = PointOrder[j];
				const int64 FlatIdx = ToFlatIdx(Idx, GridCenter, CellSize, Resolution + i*2);

				bool bAlreadyInSet = false;
				OccupiedCells.Add(FlatIdx, &bAlreadyInSet);
				if (bAlreadyInSet)
				{
					Rank[Idx - Offset] = 1;
					if (CoincidentVertices)
					{
						CoincidentVertices->Add(Idx);
					}
					NumCoincident++;
				}
			}
			if (NumCoincident > NumCoincidentPrev)
			{
				Algo::Sort(MakeArrayView(PointOrder.GetData(), NumPoints - NumCoincidentPrev), AscendingRankPred);
			}
		}
	}
	check(NumCoincident < NumPoints);
	
	// track the best points in a region, by multiple metrics (distance from center, and curvature at point)
	// note the best-curvature and best-distance points are allowed to be the same point
	struct FBestPtData
	{
		int32 FarthestIdx = -1; // Index of point farthest from center
		FReal FarthestDistSq = 0; // Squared distance of farthest point from center
		int32 MaxCurveIdx = -1; // Index of point with largest curvature
		FReal MaxCurve = (FReal)-UE_MAX_FLT; // Largest curvature value
	};
	// Update FBestPointData with a new point; return true if it's the first point in the cell, false otherwise
	auto UpdateBestPtData = [&Dist, &PointCurvatures, &Offset, &PointOrder](FBestPtData& BestData, int32 OrderIdx) -> bool
	{
		int32 PtIdx = PointOrder[OrderIdx];
		int32 OffsetPtIdx = PtIdx - Offset;
		if (BestData.FarthestIdx == -1)
		{
			BestData.FarthestIdx = OrderIdx;
			BestData.FarthestDistSq = Dist[OffsetPtIdx];
			BestData.MaxCurveIdx = OrderIdx;
			BestData.MaxCurve = PointCurvatures[OffsetPtIdx];
			return true;
		}

		if (Dist[OffsetPtIdx] > BestData.FarthestDistSq)
		{
			BestData.FarthestDistSq = Dist[OffsetPtIdx];
			BestData.FarthestIdx = OrderIdx;
		}

		FReal Curvature = PointCurvatures[OffsetPtIdx];
		if (Curvature > BestData.MaxCurve)
		{
			BestData.MaxCurve = Curvature;
			BestData.MaxCurveIdx = OrderIdx;
		}
		return false;
	};

	// Points before this offset have already been moved forward, and don't need further consideration
	int32 MovedPtsOffset = 0;
	// traverse power of 2 cell resolutions (~octree levels), moving forward "best" points from the cells at each level
	for (int32 Resolution = 2; MovedPtsOffset < NumPoints - NumCoincident && Resolution <= 1024; Resolution *= 2)
	{
		TMap<int64, FBestPtData> BestPoints;
		TSet<int64> CoveredCells, OffsetCoveredCells;
		const FReal CellSize = MaxBBoxDim / static_cast<FReal>(Resolution);

		// make smaller cells w/ a half-cell-width offset to help detect points that are close, but across cell boundaries
		int32 OffsetResolution = Resolution * 4;
		const FReal OffsetCellSize = MaxBBoxDim / static_cast<FReal>(OffsetResolution);
		OffsetResolution += 2; // allow for offset center
		FVec3 OffsetCenter = LocalCenter - FVec3(OffsetCellSize / 2);
		int32 FoundPointsCount = 0;

		// Use the Rank array to mark which points should move to the front of the remaining list in this iteration
		Rank.Reset();
		Rank.AddZeroed(NumPoints);

		auto ToMainFlatIdx = [&ToFlatIdx, &LocalCenter, &CellSize, &Resolution](int32 PointIdx)
		{
			return ToFlatIdx(PointIdx, LocalCenter, CellSize, Resolution);
		};
		auto ToOffsetFlatIdx = [&ToFlatIdx, &OffsetCenter, &OffsetCellSize, &OffsetResolution](int32 PointIdx)
		{
			return ToFlatIdx(PointIdx, OffsetCenter, OffsetCellSize, OffsetResolution);
		};

		// Mark cells that are already covered by points in the already-considered front of the list
		for (int32 OrderIdx = 0; OrderIdx < MovedPtsOffset; OrderIdx++)
		{
			int32 PointIdx = PointOrder[OrderIdx];
			const int64 FlatIdx = ToMainFlatIdx(PointIdx);
			CoveredCells.Add(FlatIdx);
			OffsetCoveredCells.Add(ToOffsetFlatIdx(PointIdx));
		}

		// find the best points in cells that aren't already covered
		bool bAllSeparate = true;
		for (int32 OrderIdx = MovedPtsOffset; OrderIdx < NumPoints - NumCoincident; OrderIdx++)
		{
			int32 PointIdx = PointOrder[OrderIdx];
			const int64 FlatIdx = ToMainFlatIdx(PointIdx);
			if (CoveredCells.Contains(FlatIdx) || OffsetCoveredCells.Contains(ToOffsetFlatIdx(PointIdx)))
			{
				bAllSeparate = false;
				continue;
			}
			FBestPtData& Best = BestPoints.FindOrAdd(FlatIdx);
			bAllSeparate = UpdateBestPtData(Best, OrderIdx) & bAllSeparate;
		}

		if (bAllSeparate)
		{
			// all the points were in separate cells, no need to continue promoting the 'best' for each cell beyond here
			break;
		}

		auto ConsiderMovingPt = [&Rank, Offset, MovedPtsOffset, &OffsetCoveredCells, &ToOffsetFlatIdx](int32 PointIdx) -> bool
		{
			uint8& R = Rank[PointIdx - Offset];
			if (!R)
			{
				// try to avoid clumping by skipping points whose offset cells were already covered
				int64 OffsetFlatIdx = ToOffsetFlatIdx(PointIdx);
				bool bAlreadyInSet = false;
				OffsetCoveredCells.Add(OffsetFlatIdx, &bAlreadyInSet);
				if (bAlreadyInSet)
				{
					return false;
				}

				R = 1;
				return true;
			}
			else
			{
				return false;
			}
		};

		// Decide which points to move into the 'front' section
		int32 NumToMoveToFront = 0;
		TArray<int32> MoveToFront;
		// Add points favored by the (more reliable) distance-from-center metric first
		for (const TPair<int64, FBestPtData>& BestDataPair : BestPoints)
		{
			int PtIdx = PointOrder[BestDataPair.Value.FarthestIdx];
			if (ConsiderMovingPt(PtIdx))
			{
				NumToMoveToFront++;
				MoveToFront.Add(BestDataPair.Value.FarthestIdx);
			}
		}
		// Add points favored by the curvature metric second
		//  (so they may be skipped if distance-prioritized ones already covered the region)
		for (const TPair<int64, FBestPtData>& BestDataPair : BestPoints)
		{
			int PtIdx = PointOrder[BestDataPair.Value.MaxCurveIdx];
			if (ConsiderMovingPt(PtIdx))
			{
				NumToMoveToFront++;
				MoveToFront.Add(BestDataPair.Value.MaxCurveIdx);
			}
		}

		// Do the move-to-front operations by swapping
		int MoveFrontIdx = 0;
		for (int MoveBackIdx = 0; MoveBackIdx < NumToMoveToFront; MoveBackIdx++)
		{
			if (Rank[PointOrder[MovedPtsOffset + MoveBackIdx] - Offset])
			{
				// Point was marked for moving to front, and is already in front; skip it
				continue;
			}
			// Swap point with one that was marked for the front section
			bool bDidSwap = false;
			for (; MoveFrontIdx < MoveToFront.Num(); MoveFrontIdx++)
			{
				if (MoveToFront[MoveFrontIdx] < MovedPtsOffset + NumToMoveToFront)
				{
					// Don't swap this one back because it's already in the 'front' zone
					continue;
				}
				int32 PtIdx = PointOrder[MoveToFront[MoveFrontIdx]];
				checkSlow(Rank[PtIdx - Offset]);
				Swap(PointOrder[MovedPtsOffset + MoveBackIdx], PointOrder[MoveToFront[MoveFrontIdx]]);
				bDidSwap = true;
				MoveFrontIdx++;
				break;
			}
			// The above for loop should always find a point to swap back
			checkSlow(bDidSwap);
		}

		// Note the above swapping method could have been done via a Sort() using the Rank array, but swapping is more direct / faster

		// Sort the just-added points by curvature (they were in arbitrary order before)
		DescendingPredicate<FReal> DescendingCurvaturePred(PointCurvatures);
		Algo::Sort(MakeArrayView(&PointOrder[MovedPtsOffset], NumToMoveToFront), DescendingCurvaturePred);

		MovedPtsOffset += NumToMoveToFront;
	}

	// Sort remaining non-coincident points by curvature
	if (MovedPtsOffset < NumPoints)
	{
		DescendingPredicate<FReal> DescendingCurvaturePred(PointCurvatures);
		Algo::Sort(MakeArrayView(&PointOrder[MovedPtsOffset], NumPoints - MovedPtsOffset - NumCoincident), DescendingCurvaturePred);
	}

	return PointOrder;
}

TArray<int32>
FTriangleMesh::GetVertexImportanceOrdering(const TConstArrayView<FVec3>& Points, TArray<int32>* CoincidentVertices, const bool RestrictToLocalIndexRange)
{
	const TArray<FReal> pointCurvatures = GetCurvatureOnPoints(Points);
	return GetVertexImportanceOrdering(Points, pointCurvatures, CoincidentVertices, RestrictToLocalIndexRange);
}

void FTriangleMesh::RemapVertices(const TArray<int32>& Order)
{
	// Remap element indices
	int32 MinIdx = TNumericLimits<int32>::Max();
	int32 MaxIdx = -TNumericLimits<int32>::Max();
	for (int32 i = 0; i < MElements.Num(); i++)
	{
		TVec3<int32>& elem = MElements[i];
		for (int32 j = 0; j < 3; ++j)
		{
			if (elem[j] != Order[elem[j]])
			{
				elem[j] = Order[elem[j]];
				MinIdx = elem[j] < MinIdx ? elem[j] : MinIdx;
				MaxIdx = elem[j] > MaxIdx ? elem[j] : MaxIdx;
			}
		}
	}
	if (MinIdx != TNumericLimits<int32>::Max())
	{
		ExpandVertexRange(MinIdx, MaxIdx);
		RemoveDuplicateElements();
		RemoveDegenerateElements();
		ResetAuxiliaryStructures();
	}
}

void FTriangleMesh::RemapVertices(const TMap<int32, int32>& Remapping)
{
	if (!Remapping.Num())
	{
		return;
	}
	int32 MinIdx = TNumericLimits<int32>::Max();
	int32 MaxIdx = -TNumericLimits<int32>::Max();
	for (TVec3<int32>& Tri : MElements)
	{
		for (int32 Idx = 0; Idx < 3; ++Idx)
		{
			if (const int32* ToIdx = Remapping.Find(Tri[Idx]))
			{
				Tri[Idx] = *ToIdx;
				MinIdx = *ToIdx < MinIdx ? *ToIdx : MinIdx;
				MaxIdx = *ToIdx > MaxIdx ? *ToIdx : MaxIdx;
			}
		}
	}
	if (MinIdx != TNumericLimits<int32>::Max())
	{
		ExpandVertexRange(MinIdx, MaxIdx);
		RemoveDuplicateElements();
		RemoveDegenerateElements();
		ResetAuxiliaryStructures();
	}
}

void FTriangleMesh::RemoveDuplicateElements()
{
	TArray<int32> ToRemove;
	TSet<TVec3<int32>> Existing;
	for (int32 Idx = 0; Idx < MElements.Num(); ++Idx)
	{
		const TVec3<int32>& Tri = MElements[Idx];
		const TVec3<int32> OrderedTri = GetOrdered(Tri);
		if (!Existing.Contains(OrderedTri))
		{
			Existing.Add(OrderedTri);
			continue;
		}
		ToRemove.Add(Idx);
	}
	for (int32 Idx = ToRemove.Num() - 1; Idx >= 0; --Idx)
	{
		MElements.RemoveAtSwap(ToRemove[Idx]);
	}
}

void FTriangleMesh::RemoveDegenerateElements()
{
	for (int i = MElements.Num() - 1; i >= 0; --i)
	{
		if (MElements[i][0] == MElements[i][1] ||
			MElements[i][0] == MElements[i][2] ||
			MElements[i][1] == MElements[i][2])
		{
			// It's possible that the order of the triangles might be important.
			// RemoveAtSwap() changes the order of the array.  I figure that if
			// you're up for CullDegenerateElements, then triangle reordering is
			// fair game.
			MElements.RemoveAtSwap(i);
		}
	}
}

template<typename T>
void FTriangleMesh::BuildBVH(const TConstArrayView<TVec3<T>>& Points, TBVHType<T>& BVH) const
{
	TArray<TTriangleMeshBvEntry<T>> BVEntries;
	const int32 NumTris = MElements.Num();
	BVEntries.Reset(NumTris);
	for (int32 Tri = 0; Tri < NumTris; ++Tri)
	{
		BVEntries.Add({ this, &Points, Tri });
	}
	BVH.Reinitialize(BVEntries);
}
template CHAOS_API void FTriangleMesh::BuildBVH<FRealSingle>(const TConstArrayView<TVec3<FRealSingle>>& Points, TBVHType<FRealSingle>& BVH) const;
template CHAOS_API void FTriangleMesh::BuildBVH<FRealDouble>(const TConstArrayView<TVec3<FRealDouble>>& Points, TBVHType<FRealDouble>& BVH) const;

template<typename T>
bool FTriangleMesh::PointProximityQuery(const TBVHType<T>& BVH, const TConstArrayView<TVec3<T>>& Points, const int32 PointIndex, const TVec3<T>& PointPosition, const T PointThickness, const T ThisThickness, 
	TFunctionRef<bool(const int32 PointIndex, const int32 TriangleIndex)> BroadphaseTest, TArray<TTriangleCollisionPoint<T>>& Result) const
{
	const T TotalThickness = ThisThickness + PointThickness;
	const T TotalThicknessSq = TotalThickness * TotalThickness;
	FAABB3 QueryBounds(PointPosition, PointPosition);
	QueryBounds.Thicken(TotalThickness);

	const TArray<int32> PotentialIntersections = BVH.FindAllIntersections(QueryBounds);

	Result.Reset(PotentialIntersections.Num());

	for (int32 TriIdx : PotentialIntersections)
	{
		if (!BroadphaseTest(PointIndex, TriIdx))
		{
			continue;
		}

		const TVec3<T>& A = Points[MElements[TriIdx][0]];
		const TVec3<T>& B = Points[MElements[TriIdx][1]];
		const TVec3<T>& C = Points[MElements[TriIdx][2]];
		TVec3<T> Bary;
		const TVec3<T> ClosestPoint = FindClosestPointAndBaryOnTriangle(A, B, C, PointPosition, Bary);

		const T DistSq = (PointPosition - ClosestPoint).SizeSquared();
		if (DistSq > TotalThicknessSq)
		{
			// Failed narrow test.
			continue;
		}

		TVec3<T> Normal = TVec3<T>::CrossProduct(B-A, C-A).GetSafeNormal();
		Normal = (TVec3<T>::DotProduct(Normal, PointPosition-A) > 0) ? Normal : -Normal;

		TTriangleCollisionPoint<T> CollisionPoint;
		CollisionPoint.ContactType = TTriangleCollisionPoint<T>::EContactType::PointFace;
		CollisionPoint.Indices[0] = PointIndex;
		CollisionPoint.Indices[1] = TriIdx;
		CollisionPoint.Bary = TVec4<T>((T)1., Bary.X, Bary.Y, Bary.Z);
		CollisionPoint.Location = ClosestPoint;
		CollisionPoint.Normal = Normal;
		CollisionPoint.Phi = FMath::Sqrt(DistSq);
		Result.Add(CollisionPoint);
	}
	return Result.Num() > 0;
}
template bool FTriangleMesh::PointProximityQuery<FRealSingle>(const TBVHType<FRealSingle>& BVH, const TConstArrayView<TVector<FRealSingle, 3>>& Points, const int32 PointIndex, const TVector<FRealSingle, 3>& PointPosition, const FRealSingle PointThickness, const FRealSingle ThisThickness, TFunctionRef<bool(const int32 PointIndex, const int32 TriangleIndex)> BroadphaseTest, TArray<TTriangleCollisionPoint<FRealSingle>>& Result) const;
template bool FTriangleMesh::PointProximityQuery<FRealDouble>(const TBVHType<FRealDouble>& BVH, const TConstArrayView<TVector<FRealDouble, 3>>& Points, const int32 PointIndex, const TVector<FRealDouble, 3>& PointPosition, const FRealDouble PointThickness, const FRealDouble ThisThickness, TFunctionRef<bool(const int32 PointIndex, const int32 TriangleIndex)> BroadphaseTest, TArray<TTriangleCollisionPoint<FRealDouble>>& Result) const;

template<typename T>
bool FTriangleMesh::EdgeIntersectionQuery(const TBVHType<T>& BVH, const TConstArrayView<TVec3<T>>& Points, const int32 EdgeIndex, const TVec3<T>& EdgePosition1, const TVec3<T>& EdgePosition2,
	TFunctionRef<bool(const int32 EdgeIndex, const int32 TriangleIndex)> BroadphaseTest, TArray<TTriangleCollisionPoint<T>>& Result) const
{
	FAABB3 QueryBounds(EdgePosition1, EdgePosition1);
	QueryBounds.GrowToInclude(EdgePosition2);

	const TArray<int32> PotentialIntersections = BVH.FindAllIntersections(QueryBounds);

	Result.Reset(PotentialIntersections.Num());

	for (int32 TriIdx : PotentialIntersections)
	{
		if (!BroadphaseTest(EdgeIndex, TriIdx))
		{
			continue;
		}

		const TVec3<T>& A = Points[MElements[TriIdx][0]];
		const TVec3<T>& B = Points[MElements[TriIdx][1]];
		const TVec3<T>& C = Points[MElements[TriIdx][2]];

		const TTriangle<T> Triangle(A, B, C);

		T Time;
		TVector<T, 2> Bary;
		if (Triangle.LineIntersection(EdgePosition1, EdgePosition2, Bary, Time))
		{
			TTriangleCollisionPoint<T> CollisionPoint;
			CollisionPoint.ContactType = TTriangleCollisionPoint<T>::EContactType::EdgeFace;
			CollisionPoint.Indices[0] = EdgeIndex;
			CollisionPoint.Indices[1] = TriIdx;
			CollisionPoint.Bary = TVec4<T>(Time, (T)1. - Bary.X - Bary.Y, Bary.X, Bary.Y);
			CollisionPoint.Location = ((T)1. - Time) * EdgePosition1 + Time * EdgePosition2;
			CollisionPoint.Normal = Triangle.GetNormal();
			CollisionPoint.Phi = 0;
			Result.Add(CollisionPoint);
		}
	}
	return Result.Num() > 0;
}
template bool FTriangleMesh::EdgeIntersectionQuery<FRealSingle>(const TBVHType<FRealSingle>& BVH, const TConstArrayView<TVec3<FRealSingle>>& Points, const int32 EdgeIndex, const TVec3<FRealSingle>& EdgePosition1, const TVec3<FRealSingle>& EdgePosition2,
	TFunctionRef<bool(const int32 EdgeIndex, const int32 TriangleIndex)> BroadphaseTest, TArray<TTriangleCollisionPoint<FRealSingle>>& Result) const;
template bool FTriangleMesh::EdgeIntersectionQuery<FRealDouble>(const TBVHType<FRealDouble>& BVH, const TConstArrayView<TVec3<FRealDouble>>& Points, const int32 EdgeIndex, const TVec3<FRealDouble>& EdgePosition1, const TVec3<FRealDouble>& EdgePosition2,
	TFunctionRef<bool(const int32 EdgeIndex, const int32 TriangleIndex)> BroadphaseTest, TArray<TTriangleCollisionPoint<FRealDouble>>& Result) const;



template<typename T>
bool FTriangleMesh::SmoothProject(
	const TBVHType<T>& BVH, 
	const TConstArrayView<FVec3>& Points,
	const TArray<FVec3>& PointNormals,
	const FVec3& Pos,
	int32& TriangleIndex,
	FVec3& Weights, 
	const int32 MaxIters) const
{
	TSet<int32> SkipTris;
	int32 Iter = 0;
	do {
		TArray<int32> CandidateTris;
		do {
			FAABB3 Box(Pos - FVec3(.5 * Iter), Pos + FVec3(.5 * Iter));
			CandidateTris = BVH.FindAllIntersections(Box);
			// Remove candidates we've already considered
			for (int32 i = CandidateTris.Num() - 1; i >= 0; i--)
			{
				if (SkipTris.Contains(CandidateTris[i]))
				{
					CandidateTris.RemoveAt(i);
				}
			}
			++Iter;
		} while (!CandidateTris.Num() && Iter < MaxIters);
		if (!CandidateTris.Num())
		{
			// If we exited the BVH loop with no candidates, then we've exceeded MaxIters.
			return false;
		}
		// Test candidates
		TArray<FVec3> CandidateWeights;
		TArray<bool> Success = Chaos::SmoothProject<T>(Points, MElements, PointNormals, Pos, CandidateTris, CandidateWeights, /*UseFirstFound = */ false);
		double MinDistanceSq = FLT_MAX; // find closest triangle
		for (int32 i = 0; i < Success.Num(); i++)
		{
			if (Success[i])
			{
				FVec3 InterpPos = FVec3(0);
				for (int32 LocalIdx = 0; LocalIdx < 3; ++LocalIdx)
				{
					InterpPos += Points[MElements[CandidateTris[i]][LocalIdx]] * CandidateWeights[i][LocalIdx];
				}
				double DistSq = (Pos - InterpPos).SizeSquared();
				if (DistSq < MinDistanceSq)
				{
					MinDistanceSq = DistSq;
					TriangleIndex = CandidateTris[i];
					Weights = CandidateWeights[i];
				}
			}
		}
		if (MinDistanceSq < FLT_MAX)
		{
			return true;
		}
		// No hits. Add candidates to skip list, and go searching for more.
		SkipTris.Append(CandidateTris);
	} while (Iter < MaxIters);
	return false;
}
template CHAOS_API bool FTriangleMesh::SmoothProject<FRealSingle>(const TBVHType<FRealSingle>& BVH, const TConstArrayView<FVec3>& Points, const TArray<FVec3>& PointNormals, const FVec3& Point, int32& TriangleIndex, FVec3& Weights, const int32 MaxIters) const;
template CHAOS_API bool FTriangleMesh::SmoothProject<FRealDouble>(const TBVHType<FRealDouble>& BVH, const TConstArrayView<FVec3>& Points, const TArray<FVec3>& PointNormals, const FVec3& Point, int32& TriangleIndex, FVec3& Weights, const int32 MaxIters) const;

template<typename T>
void FTriangleMesh::BuildSpatialHash(const TConstArrayView<TVec3<T>>& Points, TSpatialHashType<T>& SpatialHash, const T MinSpatialLodSize) const
{
	const TTriangleMeshBvData<T> BvData({ this, &Points });

	SpatialHash.Initialize(BvData, MinSpatialLodSize);
}
template CHAOS_API void FTriangleMesh::BuildSpatialHash<FRealSingle>(const TConstArrayView<TVec3<FRealSingle>>& Points, TSpatialHashType<FRealSingle>& SpatialHash, const FRealSingle MinSpatialLodSize) const;
template CHAOS_API void FTriangleMesh::BuildSpatialHash<FRealDouble>(const TConstArrayView<TVec3<FRealDouble>>& Points, TSpatialHashType<FRealDouble>& SpatialHash, const FRealDouble MinSpatialLodSize) const;

void FTriangleMesh::BuildSpatialHash(const TConstArrayView<Softs::FSolverVec3>& Points, TSpatialHashType<Softs::FSolverReal>& SpatialHash, const Softs::FPBDFlatWeightMap& PointThicknesses, int32 ThicknessMapIndexOffset, const Softs::FSolverReal MinSpatialLodSize) const
{
	const FTriangleMeshBvDataWithThickness BvData({ this, Points, PointThicknesses, ThicknessMapIndexOffset });
	SpatialHash.Initialize(BvData, MinSpatialLodSize);
}

template<typename T>
bool FTriangleMesh::PointProximityQuery(const TSpatialHashType<T>& SpatialHash, const TConstArrayView<TVec3<T>>& Points, const int32 PointIndex, const TVec3<T>& PointPosition, const T PointThickness, const T ThisThickness,
	TFunctionRef<bool(const int32 PointIndex, const int32 TriangleIndex)> BroadphaseTest, TArray<TTriangleCollisionPoint<T>>& Result) const
{
	const T TotalThickness = ThisThickness + PointThickness;
	const T TotalThicknessSq = TotalThickness * TotalThickness;
	typename TSpatialHashType<T>::FVectorAABB QueryBounds(PointPosition);
	QueryBounds.Thicken(TotalThickness);

	const TArray<int32> PotentialIntersections = SpatialHash.FindAllIntersections(QueryBounds,
		[PointIndex, &BroadphaseTest](int32 Payload)
	{
		return BroadphaseTest(PointIndex, Payload);
	});

	Result.Reset(PotentialIntersections.Num());

	for (int32 TriIdx : PotentialIntersections)
	{

		const TVec3<T>& A = Points[MElements[TriIdx][0]];
		const TVec3<T>& B = Points[MElements[TriIdx][1]];
		const TVec3<T>& C = Points[MElements[TriIdx][2]];
		TVec3<T> Bary;
		const TVec3<T> ClosestPoint = FindClosestPointAndBaryOnTriangle(A, B, C, PointPosition, Bary);

		const T DistSq = (PointPosition - ClosestPoint).SizeSquared();
		if (DistSq > TotalThicknessSq)
		{
			// Failed narrow test.
			continue;
		}

		TVec3<T> Normal = TVec3<T>::CrossProduct(B - A, C - A).GetSafeNormal();
		Normal = (TVec3<T>::DotProduct(Normal, PointPosition - A) > 0) ? Normal : -Normal;

		TTriangleCollisionPoint<T> CollisionPoint;
		CollisionPoint.ContactType = TTriangleCollisionPoint<T>::EContactType::PointFace;
		CollisionPoint.Indices[0] = PointIndex;
		CollisionPoint.Indices[1] = TriIdx;
		CollisionPoint.Bary = TVec4<T>((T)1., Bary.X, Bary.Y, Bary.Z);
		CollisionPoint.Location = ClosestPoint;
		CollisionPoint.Normal = Normal;
		CollisionPoint.Phi = FMath::Sqrt(DistSq);
		Result.Add(CollisionPoint);
	}
	return Result.Num() > 0;
}
template CHAOS_API bool FTriangleMesh::PointProximityQuery<FRealSingle>(const TSpatialHashType<FRealSingle>& SpatialHash, const TConstArrayView<TVector<FRealSingle, 3>>& Points, const int32 PointIndex, const TVector<FRealSingle, 3>& PointPosition, const FRealSingle PointThickness, const FRealSingle ThisThickness, TFunctionRef<bool(const int32 PointIndex, const int32 TriangleIndex)> BroadphaseTest, TArray<TTriangleCollisionPoint<FRealSingle>>& Result) const;
template CHAOS_API bool FTriangleMesh::PointProximityQuery<FRealDouble>(const TSpatialHashType<FRealDouble>& SpatialHash, const TConstArrayView<TVector<FRealDouble, 3>>& Points, const int32 PointIndex, const TVector<FRealDouble, 3>& PointPosition, const FRealDouble PointThickness, const FRealDouble ThisThickness, TFunctionRef<bool(const int32 PointIndex, const int32 TriangleIndex)> BroadphaseTest, TArray<TTriangleCollisionPoint<FRealDouble>>& Result) const;

bool FTriangleMesh::PointProximityQuery(const TSpatialHashType<Softs::FSolverReal>& SpatialHash, const TConstArrayView<Softs::FSolverVec3>& Points, const int32 PointIndex, const Softs::FSolverVec3& PointPosition, const Softs::FSolverReal PointThickness, const Softs::FPBDFlatWeightMap& ThisThicknesses,
	const Softs::FSolverReal ThisThicknessExtraMultiplier, int32 ThicknessMapIndexOffset, TFunctionRef<bool(const int32 PointIndex, const int32 TriangleIndex)> BroadphaseTest, TArray<TTriangleCollisionPoint<Softs::FSolverReal>>& Result) const
{
	typename TSpatialHashType<FRealSingle>::FVectorAABB QueryBounds(PointPosition);
	QueryBounds.Thicken(PointThickness);

	const TArray<int32> PotentialIntersections = SpatialHash.FindAllIntersections(QueryBounds,
		[PointIndex, &BroadphaseTest](int32 Payload)
	{
		return BroadphaseTest(PointIndex, Payload);
	});

	Result.Reset(PotentialIntersections.Num());

	for (int32 TriIdx : PotentialIntersections)
	{

		const Softs::FSolverVec3& A = Points[MElements[TriIdx][0]];
		const Softs::FSolverVec3& B = Points[MElements[TriIdx][1]];
		const Softs::FSolverVec3& C = Points[MElements[TriIdx][2]];
		Softs::FSolverVec3 Bary;
		const Softs::FSolverVec3 ClosestPoint = FindClosestPointAndBaryOnTriangle(A, B, C, PointPosition, Bary);

		const Softs::FSolverReal TriangleThickness = (ThisThicknesses.GetValue(MElements[TriIdx][0] - ThicknessMapIndexOffset) * Bary[0] +
			ThisThicknesses.GetValue(MElements[TriIdx][1] - ThicknessMapIndexOffset) * Bary[1] +
			ThisThicknesses.GetValue(MElements[TriIdx][2] - ThicknessMapIndexOffset) * Bary[2]) * ThisThicknessExtraMultiplier;

		const Softs::FSolverReal TotalThickness = PointThickness + TriangleThickness;
		const Softs::FSolverReal TotalThicknessSq = TotalThickness * TotalThickness;

		const Softs::FSolverReal DistSq = (PointPosition - ClosestPoint).SizeSquared();
		if (DistSq > TotalThicknessSq)
		{
			// Failed narrow test.
			continue;
		}

		Softs::FSolverVec3 Normal = Softs::FSolverVec3::CrossProduct(B - A, C - A).GetSafeNormal();
		Normal = (Softs::FSolverVec3::DotProduct(Normal, PointPosition - A) > 0) ? Normal : -Normal;

		TTriangleCollisionPoint<Softs::FSolverReal> CollisionPoint;
		CollisionPoint.ContactType = TTriangleCollisionPoint<Softs::FSolverReal>::EContactType::PointFace;
		CollisionPoint.Indices[0] = PointIndex;
		CollisionPoint.Indices[1] = TriIdx;
		CollisionPoint.Bary = TVec4<Softs::FSolverReal>((Softs::FSolverReal)1., Bary.X, Bary.Y, Bary.Z);
		CollisionPoint.Location = ClosestPoint;
		CollisionPoint.Normal = Normal;
		CollisionPoint.Phi = FMath::Sqrt(DistSq);
		Result.Add(CollisionPoint);
	}
	return Result.Num() > 0;

}

template<typename T>
bool FTriangleMesh::PointClosestTriangleQuery(const TSpatialHashType<T>& SpatialHash, const TConstArrayView<TVec3<T>>& Points, const int32 PointIndex, const TVec3<T>& PointPosition, const T PointThickness, const T ThisThickness,
	TFunctionRef<bool(const int32 PointIndex, const int32 TriangleIndex)> BroadphaseTest, TArray<TTriangleCollisionPoint<T>>& Result) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTriangleMesh_PointClosestTriangleQuery);
	const T TotalThickness = ThisThickness + PointThickness;
	const T TotalThicknessSq = TotalThickness * TotalThickness;
	typename TSpatialHashType<T>::FVectorAABB QueryBounds(PointPosition);
	QueryBounds.Thicken(TotalThickness);

	const TArray<int32> PotentialIntersections = SpatialHash.FindAllIntersections(QueryBounds,
		[PointIndex, &BroadphaseTest](int32 Payload)
		{
			return BroadphaseTest(PointIndex, Payload);
		});
	TRACE_CPUPROFILER_EVENT_SCOPE(FTriangleMesh_PointClosestTriangleQueryAddTriangle);
	Result.Reset(0);
	T MinDistanceSq = TotalThicknessSq;
	int32 MinTriId = INDEX_NONE;
	for (int32 TriIdx : PotentialIntersections)
	{
		const TVec3<T>& A = Points[MElements[TriIdx][0]];
		const TVec3<T>& B = Points[MElements[TriIdx][1]];
		const TVec3<T>& C = Points[MElements[TriIdx][2]];
		TVec3<T> Bary;
		const TVec3<T> ClosestPoint = FindClosestPointAndBaryOnTriangle(A, B, C, PointPosition, Bary);

		const T DistSq = (PointPosition - ClosestPoint).SizeSquared();
		if (DistSq > TotalThicknessSq || DistSq > MinDistanceSq)
		{
			// Failed narrow test.
			continue;
		}
		else
		{
			MinDistanceSq = DistSq;
			MinTriId = TriIdx;
		}
	}
	if (MinTriId != INDEX_NONE)
	{
		const TVec3<T>& A = Points[MElements[MinTriId][0]];
		const TVec3<T>& B = Points[MElements[MinTriId][1]];
		const TVec3<T>& C = Points[MElements[MinTriId][2]];
		const TTriangle<T> Triangle(A, B, C);
		TVec3<T> Bary;
		const TVec3<T> ClosestPoint = FindClosestPointAndBaryOnTriangle(A, B, C, PointPosition, Bary);
		const T DistSq = (PointPosition - ClosestPoint).SizeSquared();
		const TVec3<T> Normal = Triangle.GetNormal();
		
		TTriangleCollisionPoint<T> CollisionPoint;
		CollisionPoint.ContactType = TTriangleCollisionPoint<T>::EContactType::PointFace;
		CollisionPoint.Indices[0] = PointIndex;
		CollisionPoint.Indices[1] = MinTriId;
		CollisionPoint.Bary = TVec4<T>((T)1., Bary.X, Bary.Y, Bary.Z);
		CollisionPoint.Location = ClosestPoint;
		CollisionPoint.Normal = Normal;
		CollisionPoint.Phi = FMath::Sqrt(DistSq);
		if (TVec3<T>::DotProduct(Normal, PointPosition - A) > 0) //Point is inside of triangle (normal is pointing inward)
		{
			CollisionPoint.Phi = -CollisionPoint.Phi;
		}
		Result.Add(CollisionPoint);
	}
	return Result.Num() > 0;
}
template CHAOS_API bool FTriangleMesh::PointClosestTriangleQuery<FRealSingle>(const TSpatialHashType<FRealSingle>& SpatialHash, const TConstArrayView<TVector<FRealSingle, 3>>& Points, const int32 PointIndex, const TVector<FRealSingle, 3>& PointPosition, const FRealSingle PointThickness, const FRealSingle ThisThickness, TFunctionRef<bool(const int32 PointIndex, const int32 TriangleIndex)> BroadphaseTest, TArray<TTriangleCollisionPoint<FRealSingle>>& Result) const;
template CHAOS_API bool FTriangleMesh::PointClosestTriangleQuery<FRealDouble>(const TSpatialHashType<FRealDouble>& SpatialHash, const TConstArrayView<TVector<FRealDouble, 3>>& Points, const int32 PointIndex, const TVector<FRealDouble, 3>& PointPosition, const FRealDouble PointThickness, const FRealDouble ThisThickness, TFunctionRef<bool(const int32 PointIndex, const int32 TriangleIndex)> BroadphaseTest, TArray<TTriangleCollisionPoint<FRealDouble>>& Result) const;

template<typename T>
bool FTriangleMesh::PointClosestTriangleQuery(const TSpatialHashType<T>& SpatialHash, TFunctionRef<const TVec3<T>& (int32 PointIndex)> GetPosition, const int32 PointIndex, const TVec3<T>& PointPosition, const T PointThickness, const T ThisThickness,
	TFunctionRef<bool(const int32 PointIndex, const int32 TriangleIndex)> BroadphaseTest, TArray<TTriangleCollisionPoint<T>>& Result) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTriangleMesh_PointClosestTriangleQuery);
	const T TotalThickness = ThisThickness + PointThickness;
	const T TotalThicknessSq = TotalThickness * TotalThickness;
	typename TSpatialHashType<T>::FVectorAABB QueryBounds(PointPosition);
	QueryBounds.Thicken(TotalThickness);

	const TArray<int32> PotentialIntersections = SpatialHash.FindAllIntersections(QueryBounds,
		[PointIndex, &BroadphaseTest](int32 Payload)
		{
			return BroadphaseTest(PointIndex, Payload);
		});
	TRACE_CPUPROFILER_EVENT_SCOPE(FTriangleMesh_PointClosestTriangleQueryAddTriangle);
	Result.Reset(0);
	T MinDistanceSq = TotalThicknessSq;
	int32 MinTriId = INDEX_NONE;
	for (int32 TriIdx : PotentialIntersections)
	{
		const TVec3<T>& A = GetPosition(MElements[TriIdx][0]);
		const TVec3<T>& B = GetPosition(MElements[TriIdx][1]);
		const TVec3<T>& C = GetPosition(MElements[TriIdx][2]);
		TVec3<T> Bary;
		const TVec3<T> ClosestPoint = FindClosestPointAndBaryOnTriangle(A, B, C, PointPosition, Bary);

		const T DistSq = (PointPosition - ClosestPoint).SizeSquared();
		if (DistSq >= TotalThicknessSq || DistSq >= MinDistanceSq) // also filters 0 vs 0 case
		{
			// Failed narrow test.
			continue;
		}
		else
		{
			MinDistanceSq = DistSq;
			MinTriId = TriIdx;
		}
	}
	if (MinTriId != INDEX_NONE)
	{
		const TVec3<T>& A = GetPosition(MElements[MinTriId][0]);
		const TVec3<T>& B = GetPosition(MElements[MinTriId][1]);
		const TVec3<T>& C = GetPosition(MElements[MinTriId][2]);
		const TTriangle<T> Triangle(A, B, C);
		TVec3<T> Bary;
		const TVec3<T> ClosestPoint = FindClosestPointAndBaryOnTriangle(A, B, C, PointPosition, Bary);
		const T DistSq = (PointPosition - ClosestPoint).SizeSquared();
		const TVec3<T> Normal = Triangle.GetNormal();

		TTriangleCollisionPoint<T> CollisionPoint;
		CollisionPoint.ContactType = TTriangleCollisionPoint<T>::EContactType::PointFace;
		CollisionPoint.Indices[0] = PointIndex;
		CollisionPoint.Indices[1] = MinTriId;
		CollisionPoint.Bary = TVec4<T>((T)1., Bary.X, Bary.Y, Bary.Z);
		CollisionPoint.Location = ClosestPoint;
		CollisionPoint.Normal = Normal;
		CollisionPoint.Phi = FMath::Sqrt(DistSq);
		if (TVec3<T>::DotProduct(Normal, PointPosition - A) > 0) //Point is inside of triangle (normal is pointing inward)
		{
			CollisionPoint.Phi = -CollisionPoint.Phi;
		}
		Result.Add(CollisionPoint);
	}
	return Result.Num() > 0;
}
template CHAOS_API bool FTriangleMesh::PointClosestTriangleQuery<FRealSingle>(const TSpatialHashType<FRealSingle>& SpatialHash, TFunctionRef<const TVec3<FRealSingle>& (int32 PointIndex)> GetPosition, const int32 PointIndex, const TVector<FRealSingle, 3>& PointPosition, const FRealSingle PointThickness, const FRealSingle ThisThickness, TFunctionRef<bool(const int32 PointIndex, const int32 TriangleIndex)> BroadphaseTest, TArray<TTriangleCollisionPoint<FRealSingle>>& Result) const;
template CHAOS_API bool FTriangleMesh::PointClosestTriangleQuery<FRealDouble>(const TSpatialHashType<FRealDouble>& SpatialHash, TFunctionRef<const TVec3<FRealDouble>& (int32 PointIndex)> GetPosition, const int32 PointIndex, const TVector<FRealDouble, 3>& PointPosition, const FRealDouble PointThickness, const FRealDouble ThisThickness, TFunctionRef<bool(const int32 PointIndex, const int32 TriangleIndex)> BroadphaseTest, TArray<TTriangleCollisionPoint<FRealDouble>>& Result) const;


template<typename T>
bool FTriangleMesh::EdgeIntersectionQuery(const TSpatialHashType<T>& SpatialHash, const TConstArrayView<TVec3<T>>& Points, const int32 EdgeIndex, const TVec3<T>& EdgePosition1, const TVec3<T>& EdgePosition2,
	TFunctionRef<bool(const int32 EdgeIndex, const int32 TriangleIndex)> BroadphaseTest, TArray<TTriangleCollisionPoint<T>>& Result) const
{
	typename TSpatialHashType<T>::FVectorAABB QueryBounds(EdgePosition1);
	QueryBounds.GrowToInclude(EdgePosition2);

	const TArray<int32> PotentialIntersections = SpatialHash.FindAllIntersections(QueryBounds,
		[EdgeIndex, &BroadphaseTest](int32 Payload)
	{
		return BroadphaseTest(EdgeIndex, Payload);
	}
	);

	Result.Reset(PotentialIntersections.Num());

	for (int32 TriIdx : PotentialIntersections)
	{
		const TVec3<T>& A = Points[MElements[TriIdx][0]];
		const TVec3<T>& B = Points[MElements[TriIdx][1]];
		const TVec3<T>& C = Points[MElements[TriIdx][2]];

		const TTriangle<T> Triangle(A, B, C);

		T Time;
		TVector<T, 2> Bary;
		if (Triangle.LineIntersection(EdgePosition1, EdgePosition2, Bary, Time))
		{
			TTriangleCollisionPoint<T> CollisionPoint;
			CollisionPoint.ContactType = TTriangleCollisionPoint<T>::EContactType::EdgeFace;
			CollisionPoint.Indices[0] = EdgeIndex;
			CollisionPoint.Indices[1] = TriIdx;
			CollisionPoint.Bary = TVec4<T>(Time, (T)1. - Bary.X - Bary.Y, Bary.X, Bary.Y);
			CollisionPoint.Location = ((T)1. - Time) * EdgePosition1 + Time * EdgePosition2;
			CollisionPoint.Normal = Triangle.GetNormal();
			CollisionPoint.Phi = 0;
			Result.Add(CollisionPoint);
		}
	}
	return Result.Num() > 0;
}
template bool FTriangleMesh::EdgeIntersectionQuery<FRealSingle>(const TSpatialHashType<FRealSingle>& SpatialHash, const TConstArrayView<TVec3<FRealSingle>>& Points, const int32 EdgeIndex, const TVec3<FRealSingle>& EdgePosition1, const TVec3<FRealSingle>& EdgePosition2,
	TFunctionRef<bool(const int32 EdgeIndex, const int32 TriangleIndex)> BroadphaseTest, TArray<TTriangleCollisionPoint<FRealSingle>>& Result) const;
template bool FTriangleMesh::EdgeIntersectionQuery<FRealDouble>(const TSpatialHashType<FRealDouble>& SpatialHash, const TConstArrayView<TVec3<FRealDouble>>& Points, const int32 EdgeIndex, const TVec3<FRealDouble>& EdgePosition1, const TVec3<FRealDouble>& EdgePosition2,
	TFunctionRef<bool(const int32 EdgeIndex, const int32 TriangleIndex)> BroadphaseTest, TArray<TTriangleCollisionPoint<FRealDouble>>& Result) const;
}  // End namespace Chaos
