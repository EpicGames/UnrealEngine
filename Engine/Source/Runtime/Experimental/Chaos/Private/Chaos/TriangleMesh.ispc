// Copyright Epic Games, Inc. All Rights Reserved.

#define EXPLICIT_VECTOR4 1

#include "Math/Vector.isph"

export void GetFaceNormals(uniform FVector3f Normals[],
							const uniform FVector3f Points[],
							const uniform FIntVector Elements[],
							const uniform int32 NumElements)
{
	foreach(i = 0 ... NumElements)
	{
		const varying FIntVector Tri = VectorLoad(&Elements[extract(i, 0)]);

		const varying FVector3f p0 = VectorGather(&Points[Tri.V[0]]);
		const varying FVector3f p1 = VectorGather(&Points[Tri.V[1]]);
		const varying FVector3f p2 = VectorGather(&Points[Tri.V[2]]);

		const varying FVector3f p10 = p1 - p0;
		const varying FVector3f p20 = p2 - p0;
		const varying FVector3f Cross = VectorCross(p20, p10);
		const varying FVector3f CrossNormalized = VectorGetSafeNormal(Cross);

		VectorStore(&Normals[extract(i, 0)], CrossNormalized);
	}
}

struct TArrayInt
{
	int32 *Data;
	int32 ArrayNum;
};

export void GetPointNormals(uniform FVector3f PointNormals[],
							const uniform FVector3f FaceNormals[],
							const uniform TArrayInt PointToTriangleMap[],
							const uniform int32 StartIndex,
							const uniform int32 NumFaceNormals,
							const uniform int32 NumIndices)
{
	#pragma unroll 4
	for(uniform int32 Element = 0; Element < NumIndices; Element++)
	{
		uniform FVector4f Normal = FloatZero;

		const uniform TArrayInt TriangleMap = PointToTriangleMap[Element];  // Access MPointToTriangleMap with local index

		#pragma unroll 2
		for (uniform int32 k = 0; k < TriangleMap.ArrayNum; ++k)
		{
			const uniform int32 Index = TriangleMap.Data[k];

			if (Index >= 0 && Index < NumFaceNormals)
			{
				const uniform FVector3f FaceNormal = FaceNormals[Index];

				Normal = Normal + SetVector4(FaceNormal, FLOAT_ZERO);
			}
		}

		uniform FVector3f SafeNormal = VectorGetSafeNormal(SetVector(Normal));
		PointNormals[Element + StartIndex] = SafeNormal;
	}
}
