// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/TriangleMeshImplicitObject.h"
#include "Chaos/Collision/ContactPoint.h"
#include "Chaos/Collision/ContactPointsMiscShapes.h"
#include "Chaos/Collision/MeshContactGenerator.h"
#include "Chaos/Collision/TriangleOverlap.h"
#include "Chaos/Collision/PBDCollisionConstraint.h"
#include "Chaos/CollisionOneShotManifolds.h"
#include "Chaos/Capsule.h"
#include "Chaos/GJK.h"
#include "Chaos/Triangle.h"
#include "Chaos/TriangleRegister.h"
#include "Chaos/Convex.h"
#include "Chaos/ImplicitObjectScaled.h"
#include "Chaos/GeometryQueries.h"
#include "Chaos/Utilities.h"


namespace Chaos
{
	namespace CVars
	{
		extern bool bCCDNewTargetDepthMode;
	}

	extern FRealSingle Chaos_Collision_EdgePrunePlaneDistance;

	// Note that if this is re-enabled when previously off, the cooked trimeshes won't have the vertex map serialized, so the change will not take effect until re-cooked.
	bool TriMeshPerPolySupport = 1;
	FAutoConsoleVariableRef CVarPerPolySupport(TEXT("p.Chaos.TriMeshPerPolySupport"), TriMeshPerPolySupport, TEXT("Disabling removes memory cost of vertex map on triangle mesh. Note: Changing at runtime will not work."));

	FReal GetWindingOrder(const FVec3& Scale)
	{
		const FVec3 SignVector = Scale.GetSignVector();
		return SignVector.X * SignVector.Y * SignVector.Z;
	}

template <typename QueryGeomType>
static auto MakeScaledHelper(const QueryGeomType& B, const FVec3& InvScale)
{
	// TODO: Fixup code using this and remove it.
	return TImplicitObjectScaled<QueryGeomType,true>(const_cast<QueryGeomType*>(&B), InvScale);
}

template <typename QueryGeomType>
static auto MakeScaledHelper(const TImplicitObjectScaled<QueryGeomType>& B, const FVec3& InvScale)
{
	//if scaled of scaled just collapse into one scaled
	TImplicitObjectScaled<QueryGeomType> ScaledB(B.Object(),  InvScale * B.GetScale());   
	return ScaledB;
}

void ScaleTransformHelper(const FVec3& TriMeshScale, const FRigidTransform3& QueryTM, FRigidTransform3& OutScaledQueryTM)
{
	OutScaledQueryTM = TRigidTransform<FReal, 3>(QueryTM.GetLocation() * TriMeshScale, QueryTM.GetRotation());
}


template <typename QueryGeomType>
const QueryGeomType& ScaleGeomIntoWorldHelper(const QueryGeomType& QueryGeom, const FVec3& TriMeshScale)
{
	return QueryGeom;
}

template <typename QueryGeomType>
TImplicitObjectScaled<QueryGeomType> ScaleGeomIntoWorldHelper(const TImplicitObjectScaled<QueryGeomType>& QueryGeom, const FVec3& TriMeshScale)
{
	// This will apply TriMeshScale to QueryGeom and return a new scaled implicit in world space.
	return MakeScaledHelper(QueryGeom, TriMeshScale);
}

void TransformSweepOutputsHelper(FVec3 TriMeshScale, const FVec3& HitNormal, const FVec3& HitPosition, const FReal LengthScale,
	const FReal Time, const bool bOutputIncludesScale, FVec3& OutNormal, FVec3& OutPosition, FReal& OutTime)
{
	// To compute correct position/normal, we tested in the scaled space.
	// If we want to return the unscaled result we have to undo this scaling, otherwise we output as is.
	if (!bOutputIncludesScale)
	{
		if (ensure(TriMeshScale != FVec3(0.0f)))
		{
			FVec3 InvTriMeshScale = 1.0f / TriMeshScale;
			OutTime = Time / LengthScale;
			OutNormal = (TriMeshScale * HitNormal).GetSafeNormal();
			OutPosition = InvTriMeshScale * HitPosition;
		}
	}
	else
	{
		OutTime = Time;
		OutNormal = HitNormal.GetSafeNormal();
		OutPosition = HitPosition;
	}
}

template <typename QueryGeomType>
const auto MakeTriangleHelper(const QueryGeomType& Geom)
{
	return FPBDCollisionConstraint::MakeTriangle(&Geom);
}

// collapse scale object into it's inner shape if scale is 1, because MakeTRionagle need to be able to infer properties on the shape
template <typename QueryGeomType>
const auto MakeTriangleHelper(const TImplicitObjectScaled<QueryGeomType>& ScaledGeom)
{
	if (FVec3::IsNearlyEqual(ScaledGeom.GetScale(), FVec3(1), UE_SMALL_NUMBER))
	{
		return FPBDCollisionConstraint::MakeTriangle(ScaledGeom.GetUnscaledObject());
	}
	return FPBDCollisionConstraint::MakeTriangle(&ScaledGeom);
}

template <typename IdxType>
struct FTriangleMeshRaycastVisitor
{
	using ParticlesType = FTriangleMeshImplicitObject::ParticlesType;

	FTriangleMeshRaycastVisitor(const FVec3& InStart, const FVec3& InDir, const FReal InThickness, const ParticlesType& InParticles, const TArray<TVector<IdxType, 3>>& InElements, bool bInCullsBackFaceRaycast)
	: Particles(InParticles)
	, Elements(InElements)
	, StartPoint(InStart)
	, Dir(InDir)
	, Thickness(InThickness)
	, OutTime(TNumericLimits<FReal>::Max())
	, bCullsBackFaceRaycast(bInCullsBackFaceRaycast)
	{
	}

	enum class ERaycastType
	{
		Raycast,
		Sweep
	};

	const void* GetQueryData() const
	{
		return nullptr;
	}

	const void* GetSimData() const
	{
		return nullptr;
	}
	
	/** Return a pointer to the payload on which we are querying the acceleration structure */
	const void* GetQueryPayload() const
	{
		return nullptr;
	}

	/**
	 * find the intersection of a ray and the triangle at index TriIdx
	 * @return true if the search should continue otherwise false to stop the search through the mesh 
	 */
	bool VisitRaycast(TSpatialVisitorData<int32> TriIdx, FRealSingle& CurDataLength)
	{
		constexpr FReal Epsilon = UE_SMALL_NUMBER;

		const int32 FaceIndex = TriIdx.Payload;
		const FVec3& A = Particles.GetX(Elements[FaceIndex][0]);
		const FVec3& B = Particles.GetX(Elements[FaceIndex][1]);
		const FVec3& C = Particles.GetX(Elements[FaceIndex][2]);

		// Note: the math here needs to match FTriangleMeshImplicitObject::GetFaceNormal
		// @todo(chaos) we should really preprocess the face and remove the degenerated ones to avoid paying this runtime cost
		const FVec3 AB = B - A;
		const FVec3 AC = C - A;
		FVec3 TriNormal = FVec3::CrossProduct(AB, AC);

		if (bCullsBackFaceRaycast)
		{
			const bool bBackFace = (FVec3::DotProduct(Dir, TriNormal) > 0.0f);
			if (bBackFace)
			{
				// skip this traingle and continue the visit
				return true;
			}
	}

		FVec3 HitNormal;
		FReal HitTime;
		if (RayTriangleIntersection(StartPoint, Dir, static_cast<FReal>(CurDataLength), A, B, C, HitTime, HitNormal))
		{
			OutPosition = StartPoint + (Dir * HitTime);
			OutNormal = HitNormal;
			OutTime = HitTime;
			OutFaceIndex = FaceIndex;
			CurDataLength = static_cast<FRealSingle>(HitTime); //prevent future rays from going any farther
		}
		// continue the visit
		return true;
	}
	
	bool VisitSweep(TSpatialVisitorData<int32> TriIdx, FRealSingle& CurDataLength)
	{
		constexpr FReal Epsilon = UE_SMALL_NUMBER;
		constexpr FReal Epsilon2 = Epsilon * Epsilon;
		
		const FReal Thickness2 = Thickness * Thickness;
		FReal MinTime = 0;	//no need to initialize, but fixes warning

		const FReal R = Thickness + Epsilon;
		const FReal R2 = R * R;

		const int32 FaceIndex = TriIdx.Payload;		
		const FVec3& A = Particles.GetX(Elements[FaceIndex][0]);
		const FVec3& B = Particles.GetX(Elements[FaceIndex][1]);
		const FVec3& C = Particles.GetX(Elements[FaceIndex][2]);

		// Note: the math here needs to match FTriangleMeshImplicitObject::GetFaceNormal
		const FVec3 AB = B - A;
		const FVec3 AC = C - A;
		FVec3 TriNormal = FVec3::CrossProduct(AB, AC);
		const FReal NormalLength = TriNormal.SafeNormalize(UE_SMALL_NUMBER);
		if (!CHAOS_ENSURE(NormalLength > Epsilon))
		{
			//hitting degenerate triangle so keep searching - should be fixed before we get to this stage
			return true;
		}

		const bool bBackFace = (FVec3::DotProduct(Dir, TriNormal) > 0.0f);
		if (bCullsBackFaceRaycast && bBackFace)
		{
			return true;
		}

		const TPlane<FReal, 3> TriPlane{ A, TriNormal };
		FVec3 RaycastPosition;
		FVec3 RaycastNormal;
		FReal Time;

		//Check if we even intersect with triangle plane
		int32 DummyFaceIndex;
		if (TriPlane.Raycast(StartPoint, Dir, CurDataLength, Thickness, Time, RaycastPosition, RaycastNormal, DummyFaceIndex))
		{
			FVec3 IntersectionPosition = RaycastPosition;
			FVec3 IntersectionNormal = RaycastNormal;
			bool bTriangleIntersects = false;
			if (Time == 0)
			{
				//Initial overlap so no point of intersection, do an explicit sphere triangle test.
				const FVec3 ClosestPtOnTri = FindClosestPointOnTriangle(TriPlane, A, B, C, StartPoint);
				const FReal DistToTriangle2 = (StartPoint - ClosestPtOnTri).SizeSquared();
				if (DistToTriangle2 <= R2)
				{
					OutTime = 0;
					OutFaceIndex = FaceIndex;
					return false; //no one will beat Time == 0
				}
			}
			else
			{
				const FVec3 ClosestPtOnTri = FindClosestPointOnTriangle(RaycastPosition, A, B, C, RaycastPosition);	//We know Position is on the triangle plane
				const FReal DistToTriangle2 = (RaycastPosition - ClosestPtOnTri).SizeSquared();
				bTriangleIntersects = DistToTriangle2 <= Epsilon2;	//raycast gave us the intersection point so sphere radius is already accounted for
			}

			if (!bTriangleIntersects)
			{
				//sphere is not immediately touching the triangle, but it could start intersecting the perimeter as it sweeps by
				FVec3 BorderPositions[3];
				FVec3 BorderNormals[3];
				FReal BorderTimes[3];
				bool bBorderIntersections[3];

				{
					FVec3 ABCapsuleAxis = B - A;
					FReal ABHeight = ABCapsuleAxis.SafeNormalize();
					bBorderIntersections[0] = FCapsule::RaycastFast(Thickness, ABHeight, ABCapsuleAxis, A, B, StartPoint, Dir, CurDataLength, 0, BorderTimes[0], BorderPositions[0], BorderNormals[0], DummyFaceIndex);
				}
				
				{
					FVec3 BCCapsuleAxis = C - B;
					FReal BCHeight = BCCapsuleAxis.SafeNormalize();
					bBorderIntersections[1] = FCapsule::RaycastFast(Thickness, BCHeight, BCCapsuleAxis, B, C, StartPoint, Dir, CurDataLength, 0, BorderTimes[1], BorderPositions[1], BorderNormals[1], DummyFaceIndex);
				}
				
				{
					FVec3 ACCapsuleAxis = C - A;
					FReal ACHeight = ACCapsuleAxis.SafeNormalize();
					bBorderIntersections[2] = FCapsule::RaycastFast(Thickness, ACHeight, ACCapsuleAxis, A, C, StartPoint, Dir, CurDataLength, 0, BorderTimes[2], BorderPositions[2], BorderNormals[2], DummyFaceIndex);
				}

				int32 MinBorderIdx = INDEX_NONE;
				FReal MinBorderTime = 0;	//initialization not needed, but fixes warning

				for (int32 BorderIdx = 0; BorderIdx < 3; ++BorderIdx)
				{
					if (bBorderIntersections[BorderIdx])
					{
						if (!bTriangleIntersects || BorderTimes[BorderIdx] < MinBorderTime)
						{
							MinBorderTime = BorderTimes[BorderIdx];
							MinBorderIdx = BorderIdx;
							bTriangleIntersects = true;
						}
					}
				}

				if (MinBorderIdx != INDEX_NONE)
				{
					IntersectionNormal = BorderNormals[MinBorderIdx];
					IntersectionPosition = BorderPositions[MinBorderIdx] - IntersectionNormal * Thickness;

					if (Time == 0)
					{
						//we were initially overlapping with triangle plane so no normal was given. Compute it now
						FVec3 TmpNormal;
						const FReal SignedDistance = TriPlane.PhiWithNormal(StartPoint, TmpNormal);
						RaycastNormal = SignedDistance >= 0 ? TmpNormal : -TmpNormal;
					}

					Time = MinBorderTime;
				}
			}

			if (bTriangleIntersects)
			{
				if (Time < OutTime)
				{
					OutPosition = IntersectionPosition;
					OutNormal = RaycastNormal;	//We use the plane normal even when hitting triangle edges. This is to deal with triangles that approximate a single flat surface.
					OutTime = Time;
					CurDataLength = static_cast<FRealSingle>(Time);	//prevent future rays from going any farther
					OutFaceIndex = FaceIndex;
				}
			}
		}

		return true;
	}

	bool VisitOverlap(TSpatialVisitorData<int32> TriIdx)
	{
		check(false);
		return true;
	}

	const ParticlesType& Particles;
	const TArray<TVector<IdxType, 3>>& Elements;
	const FVec3& StartPoint;
	const FVec3& Dir;
	const FReal Thickness;
	FReal OutTime;
	FVec3 OutPosition;
	FVec3 OutNormal;
	int32 OutFaceIndex;
	bool bCullsBackFaceRaycast;
	TArray<int32> RaycastTriangles;
};

struct FTriangleMeshOverlapVisitor
{
	FTriangleMeshOverlapVisitor(TArray<int32>& InResults) : CollectedResults(InResults) {}
	bool VisitOverlap(int32 Instance)
	{
		CollectedResults.Add(Instance);
		return true;
	}
	bool VisitSweep(int32 Instance, FRealSingle& CurDataLength)
	{
		check(false);
		return true;
	}
	bool VisitRaycast(int32 Instance, FRealSingle& CurDataLength)
	{
		check(false);
		return true;
	}

	TArray<int32>& CollectedResults;
};

template <typename QueryGeomType>
struct FTriangleMeshOverlapVisitorNoMTD
{
	FTriangleMeshOverlapVisitorNoMTD(const TRigidTransform<FReal, 3>& WorldScaleQueryTM, const QueryGeomType& InQueryGeom, FReal InThickness, const FVec3& InTriMeshScale, const FTriangleMeshImplicitObject* InTriMesh)
		: QueryGeom(InQueryGeom)
		, TriMeshScale(InTriMeshScale)
		, Thickness(FRealSingle(InThickness))
		, TriMesh(InTriMesh)
		, bFoundIntersection(false)
	{
		const UE::Math::TQuat<FReal>& RotationDouble = WorldScaleQueryTM.GetRotation();
		Rotation = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(RotationDouble.X, RotationDouble.Y, RotationDouble.Z, RotationDouble.W));
		// Normalize rotation
		Rotation = VectorNormalizeSafe(Rotation, GlobalVectorConstants::Float0001);

		const UE::Math::TVector<FReal>& TranslationDouble = WorldScaleQueryTM.GetTranslation();
		Translation = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(TranslationDouble.X, TranslationDouble.Y, TranslationDouble.Z, 0.0));
	}

	bool VisitOverlap(int32 TriIdx)
	{
		FVec3 A, B, C;
		if (TriMesh->MElements.RequiresLargeIndices())
		{
			TriangleMeshTransformVertsHelper(TriMeshScale, TriIdx, TriMesh->MParticles, TriMesh->MElements.GetLargeIndexBuffer(), A, B, C);
		}
		else
		{
			TriangleMeshTransformVertsHelper(TriMeshScale, TriIdx, TriMesh->MParticles, TriMesh->MElements.GetSmallIndexBuffer(), A, B, C);
		}

		const VectorRegister4Float ASimd = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(A.X, A.Y, A.Z, 0.0));
		const VectorRegister4Float BSimd = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(B.X, B.Y, B.Z, 0.0));
		const VectorRegister4Float CSimd = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(C.X, C.Y, C.Z, 0.0));

		const VectorRegister4Float AB = VectorSubtract(BSimd, ASimd);
		const VectorRegister4Float AC = VectorSubtract(CSimd, ASimd);

		//It's most likely that the query object is in front of the triangle since queries tend to be on the outside.
		//However, maybe we should check if it's behind the triangle plane. Also, we should enforce this winding in some way
		const VectorRegister4Float InitialDir = VectorCross(AB, AC);

		FTriangleRegister Tri(ASimd, BSimd, CSimd);

		if (GJKIntersectionSimd(Tri, QueryGeom, Translation, Rotation, Thickness, InitialDir))
		{
			bFoundIntersection = true;
			return false;
		}
		return true;		
	}

	bool VisitSweep(int32 Instance, FRealSingle& CurDataLength)
	{
		check(false);
		return true;
	}
	bool VisitRaycast(int32 Instance, FRealSingle& CurDataLength)
	{
		check(false);
		return true;
	}

	VectorRegister4Float Rotation;
	VectorRegister4Float Translation;
	const QueryGeomType& QueryGeom;
	const FVec3 TriMeshScale;
	FRealSingle Thickness;
	const FTriangleMeshImplicitObject* TriMesh;
	bool bFoundIntersection;
};
namespace
{
template <typename IdxType, typename ParticlesType>
inline void TriangleMeshTransformVertsHelperSimd(const VectorRegister4Float& TriMeshScale, int32 TriIdx, const ParticlesType& Particles,
	const TArray<TVector<IdxType, 3>>& Elements, VectorRegister4Float& OutA, VectorRegister4Float& OutB, VectorRegister4Float& OutC)
{
	OutA = VectorLoadFloat3(&Particles.GetX(Elements[TriIdx][0]).X);
	OutB = VectorLoadFloat3(&Particles.GetX(Elements[TriIdx][1]).X);
	OutC = VectorLoadFloat3(&Particles.GetX(Elements[TriIdx][2]).X);
	OutA = VectorMultiply(OutA, TriMeshScale);
	OutB = VectorMultiply(OutB, TriMeshScale);
	OutC = VectorMultiply(OutC, TriMeshScale);
}
}

template <>
struct FTriangleMeshOverlapVisitorNoMTD<FCapsule>
{
	FTriangleMeshOverlapVisitorNoMTD(const TRigidTransform<FReal, 3>& WorldScaleQueryTM, const FCapsule& InQueryGeom, FReal InThickness, const FVec3f& InTriMeshScale, const FTriangleMeshImplicitObject* InTriMesh)
		: Radius(InQueryGeom.GetRadiusf() + FRealSingle(InThickness))
		, TriMesh(InTriMesh)
		, bFoundIntersection(false)
	{
		const FVec3f X1f = WorldScaleQueryTM.TransformPosition(FVec3(InQueryGeom.GetX1f()));
		const FVec3f X2f = WorldScaleQueryTM.TransformPosition(FVec3(InQueryGeom.GetX2f()));
		X1 = VectorLoadFloat3(&X1f.X);
		X2 = VectorLoadFloat3(&X2f.X);
		TriMeshScale = VectorLoadFloat3(&InTriMeshScale.X);
	}

	bool VisitOverlap(int32 TriIdx)
	{
		VectorRegister4Float A, B, C;
		if (TriMesh->MElements.RequiresLargeIndices())
		{
			TriangleMeshTransformVertsHelperSimd(TriMeshScale, TriIdx, TriMesh->MParticles, TriMesh->MElements.GetLargeIndexBuffer(), A, B, C);
		}
		else
		{
			TriangleMeshTransformVertsHelperSimd(TriMeshScale, TriIdx, TriMesh->MParticles, TriMesh->MElements.GetSmallIndexBuffer(), A, B, C);
		}

		bFoundIntersection = ComputeCapsuleTriangleOverlapSimd(A, B, C, X1, X2, Radius);
		return !bFoundIntersection;
	}

	bool VisitSweep(int32 Instance, FRealSingle& CurDataLength)
	{
		check(false);
		return true;
	}
	bool VisitRaycast(int32 Instance, FRealSingle& CurDataLength)
	{
		check(false);
		return true;
	}

	FRealSingle Radius;
	VectorRegister4Float X1;
	VectorRegister4Float X2;
	VectorRegister4Float TriMeshScale;
	const FTriangleMeshImplicitObject* TriMesh;
	bool bFoundIntersection;
};


template <>
struct FTriangleMeshOverlapVisitorNoMTD<TImplicitObjectScaled<FCapsule>>
{
	FTriangleMeshOverlapVisitorNoMTD(const TRigidTransform<FReal, 3>& WorldScaleQueryTM, const TImplicitObjectScaled< FCapsule >& InQueryGeom, FReal InThickness, const FVec3f& InTriMeshScale, const FTriangleMeshImplicitObject* InTriMesh)
		: Radius(FRealSingle(InQueryGeom.GetRadius()+ InThickness))
		, TriMesh(InTriMesh)
		, bFoundIntersection(false)
	{
		const FVec3f X1f = WorldScaleQueryTM.TransformPosition(FVec3(InQueryGeom.GetUnscaledObject()->GetX1f() * InQueryGeom.GetScale()));
		const FVec3f X2f = WorldScaleQueryTM.TransformPosition(FVec3(InQueryGeom.GetUnscaledObject()->GetX2f() * InQueryGeom.GetScale()));
		X1 = VectorLoadFloat3(&X1f.X);
		X2 = VectorLoadFloat3(&X2f.X);
		TriMeshScale = VectorLoadFloat3(&InTriMeshScale.X);
	}

	bool VisitOverlap(int32 TriIdx)
	{
		VectorRegister4Float A, B, C;
		if (TriMesh->MElements.RequiresLargeIndices())
		{
			TriangleMeshTransformVertsHelperSimd(TriMeshScale, TriIdx, TriMesh->MParticles, TriMesh->MElements.GetLargeIndexBuffer(), A, B, C);
		}
		else
		{
			TriangleMeshTransformVertsHelperSimd(TriMeshScale, TriIdx, TriMesh->MParticles, TriMesh->MElements.GetSmallIndexBuffer(), A, B, C);
		}

		bFoundIntersection = ComputeCapsuleTriangleOverlapSimd(A, B, C, X1, X2, Radius);
		return !bFoundIntersection;
	}

	bool VisitSweep(int32 Instance, FRealSingle& CurDataLength)
	{
		check(false);
		return true;
	}
	bool VisitRaycast(int32 Instance, FRealSingle& CurDataLength)
	{
		check(false);
		return true;
	}

	FRealSingle Radius;
	VectorRegister4Float X1;
	VectorRegister4Float X2;
	VectorRegister4Float TriMeshScale;
	const FTriangleMeshImplicitObject* TriMesh;
	bool bFoundIntersection;
};

template <>
struct FTriangleMeshOverlapVisitorNoMTD<Chaos::FSphere>
{
	FTriangleMeshOverlapVisitorNoMTD(const TRigidTransform<FReal, 3>& WorldScaleQueryTM, const Chaos::FSphere& InQueryGeom, FReal InThickness, const FVec3f& InTriMeshScale, const FTriangleMeshImplicitObject* InTriMesh)
		: Radius(InQueryGeom.GetRadiusf() + FRealSingle(InThickness))
		, TriMesh(InTriMesh)
		, bFoundIntersection(false)
	{
		const FVec3f Xf = WorldScaleQueryTM.TransformPosition(FVec3(InQueryGeom.GetCenterf()));
		X = VectorLoadFloat3(&Xf.X);
		TriMeshScale = VectorLoadFloat3(&InTriMeshScale.X);
	}

	bool VisitOverlap(int32 TriIdx)
	{
		VectorRegister4Float A, B, C;
		if (TriMesh->MElements.RequiresLargeIndices())
		{
			TriangleMeshTransformVertsHelperSimd(TriMeshScale, TriIdx, TriMesh->MParticles, TriMesh->MElements.GetLargeIndexBuffer(), A, B, C);
		}
		else
		{
			TriangleMeshTransformVertsHelperSimd(TriMeshScale, TriIdx, TriMesh->MParticles, TriMesh->MElements.GetSmallIndexBuffer(), A, B, C);
		}

		bFoundIntersection = ComputeSphereTriangleOverlapSimd(A, B, C, X, Radius);
		return !bFoundIntersection;
	}

	bool VisitSweep(int32 Instance, FRealSingle& CurDataLength)
	{
		check(false);
		return true;
	}
	bool VisitRaycast(int32 Instance, FRealSingle& CurDataLength)
	{
		check(false);
		return true;
	}

	FRealSingle Radius;
	VectorRegister4Float X;
	VectorRegister4Float TriMeshScale;
	const FTriangleMeshImplicitObject* TriMesh;
	bool bFoundIntersection;
};


template <>
struct FTriangleMeshOverlapVisitorNoMTD<TImplicitObjectScaled<Chaos::FSphere>>
{
	FTriangleMeshOverlapVisitorNoMTD(const TRigidTransform<FReal, 3>& WorldScaleQueryTM, const TImplicitObjectScaled< Chaos::FSphere >& InQueryGeom, FReal InThickness, const FVec3f& InTriMeshScale, const FTriangleMeshImplicitObject* InTriMesh)
		: Radius(FRealSingle(InQueryGeom.GetRadius() + InThickness))
		, TriMesh(InTriMesh)
		, bFoundIntersection(false)
	{
		const FVec3f Xf = WorldScaleQueryTM.TransformPosition(FVec3(InQueryGeom.GetUnscaledObject()->GetCenterf()) * InQueryGeom.GetScale());
		X = VectorLoadFloat3(&Xf.X);
		TriMeshScale = VectorLoadFloat3(&InTriMeshScale.X);
		FReal R = InQueryGeom.GetRadius();
		FRealSingle Rf = InQueryGeom.GetRadiusf();
		ensure(Rf == R);
	}

	bool VisitOverlap(int32 TriIdx)
	{
		VectorRegister4Float A, B, C;
		if (TriMesh->MElements.RequiresLargeIndices())
		{
			TriangleMeshTransformVertsHelperSimd(TriMeshScale, TriIdx, TriMesh->MParticles, TriMesh->MElements.GetLargeIndexBuffer(), A, B, C);
		}
		else
		{
			TriangleMeshTransformVertsHelperSimd(TriMeshScale, TriIdx, TriMesh->MParticles, TriMesh->MElements.GetSmallIndexBuffer(), A, B, C);
		}

		bFoundIntersection = ComputeSphereTriangleOverlapSimd(A, B, C, X, Radius);
		return !bFoundIntersection;
	}

	bool VisitSweep(int32 Instance, FRealSingle& CurDataLength)
	{
		check(false);
		return true;
	}
	bool VisitRaycast(int32 Instance, FRealSingle& CurDataLength)
	{
		check(false);
		return true;
	}

	FRealSingle Radius;
	VectorRegister4Float X;
	VectorRegister4Float TriMeshScale;
	const FTriangleMeshImplicitObject* TriMesh;
	bool bFoundIntersection;
};

template <>
struct FTriangleMeshOverlapVisitorNoMTD<TBox<FReal, 3>>
{
	FTriangleMeshOverlapVisitorNoMTD(const TRigidTransform<FReal, 3>& WorldScaleQueryTM, const TBox<FReal, 3>& InQueryGeom, FReal InThickness, const FVec3f& InTriMeshScale, const FTriangleMeshImplicitObject* InTriMesh)
		: bIsAxisAligned(FMath::Abs(WorldScaleQueryTM.GetRotation().W) == 1.0)
		, TriMesh(InTriMesh)
		, bFoundIntersection(false)
	{
		if (bIsAxisAligned)
		{
			Box.AABBSimd = FAABBSimd(WorldScaleQueryTM.GetTranslation(), InQueryGeom);
		}
		else
		{
			Box.BoxSimd = FBoxSimd(WorldScaleQueryTM, InQueryGeom);
		}
		
		TriMeshScale = VectorLoadFloat3(&InTriMeshScale.X);
	}

	bool VisitOverlap(int32 TriIdx)
	{
		VectorRegister4Float A, B, C;
		if (TriMesh->MElements.RequiresLargeIndices())
		{
			TriangleMeshTransformVertsHelperSimd(TriMeshScale, TriIdx, TriMesh->MParticles, TriMesh->MElements.GetLargeIndexBuffer(), A, B, C);
		}
		else
		{
			TriangleMeshTransformVertsHelperSimd(TriMeshScale, TriIdx, TriMesh->MParticles, TriMesh->MElements.GetSmallIndexBuffer(), A, B, C);
		}

		if (bIsAxisAligned)
		{
			bFoundIntersection = Box.AABBSimd.OverlapTriangle(A, B, C);
		}
		else
		{
			bFoundIntersection = Box.BoxSimd.OverlapTriangle(A, B, C);
		}
		return !bFoundIntersection;
	}

	bool VisitSweep(int32 Instance, FRealSingle& CurDataLength)
	{
		check(false);
		return true;
	}
	bool VisitRaycast(int32 Instance, FRealSingle& CurDataLength)
	{
		check(false);
		return true;
	}

	const bool bIsAxisAligned;
	struct {
		FBoxSimd  BoxSimd;
		FAABBSimd AABBSimd;
	} Box;
	VectorRegister4Float TriMeshScale;
	const FTriangleMeshImplicitObject* TriMesh;
	bool bFoundIntersection;
};


template <>
struct FTriangleMeshOverlapVisitorNoMTD<TImplicitObjectScaled<TBox<FReal, 3>>>
{
	FTriangleMeshOverlapVisitorNoMTD(const TRigidTransform<FReal, 3>& WorldScaleQueryTM, const TImplicitObjectScaled< TBox<FReal, 3> >& InQueryGeom, FReal InThickness, const FVec3f& InTriMeshScale, const FTriangleMeshImplicitObject* InTriMesh)
		: bIsAxisAligned(FMath::Abs(WorldScaleQueryTM.GetRotation().W) == 1.0)
		, TriMesh(InTriMesh)
		, bFoundIntersection(false)
	{
		if (bIsAxisAligned)
		{
			Box.AABBSimd = FAABBSimd(WorldScaleQueryTM.GetTranslation(), InQueryGeom);
		}
		else
		{
			Box.BoxSimd = FBoxSimd(WorldScaleQueryTM, InQueryGeom);
		}

		TriMeshScale = VectorLoadFloat3(&InTriMeshScale.X);
	}

	bool VisitOverlap(int32 TriIdx)
	{
		VectorRegister4Float A, B, C;
		if (TriMesh->MElements.RequiresLargeIndices())
		{
			TriangleMeshTransformVertsHelperSimd(TriMeshScale, TriIdx, TriMesh->MParticles, TriMesh->MElements.GetLargeIndexBuffer(), A, B, C);
		}
		else
		{
			TriangleMeshTransformVertsHelperSimd(TriMeshScale, TriIdx, TriMesh->MParticles, TriMesh->MElements.GetSmallIndexBuffer(), A, B, C);
		}

		if (bIsAxisAligned)
		{
			bFoundIntersection = Box.AABBSimd.OverlapTriangle(A, B, C);
		}
		else
		{
			bFoundIntersection = Box.BoxSimd.OverlapTriangle(A, B, C);
		}
		return !bFoundIntersection;
	}

	bool VisitSweep(int32 Instance, FRealSingle& CurDataLength)
	{
		check(false);
		return true;
	}
	bool VisitRaycast(int32 Instance, FRealSingle& CurDataLength)
	{
		check(false);
		return true;
	}

	const bool bIsAxisAligned;
	struct {
		FBoxSimd  BoxSimd;
		FAABBSimd AABBSimd;
	} Box;
	VectorRegister4Float TriMeshScale;
	const FTriangleMeshImplicitObject* TriMesh;
	bool bFoundIntersection;
};

template <typename QueryGeomType>
bool FTrimeshBVH::FindAllIntersectionsNoMTD(const FAABB3& Intersection, const TRigidTransform<FReal, 3>& Transform, const QueryGeomType& QueryGeom, FReal Thickness, const FVec3& TriMeshScale, const FTriangleMeshImplicitObject* TriMesh) const
{
	FTriangleMeshOverlapVisitorNoMTD Visitor(Transform, QueryGeom, Thickness, TriMeshScale, TriMesh);
	Overlap(FAABBVectorized(Intersection), Visitor);

	return Visitor.bFoundIntersection;
}

template <typename QueryGeomType>
bool FTrimeshBVH::FindAllIntersectionsNoMTD(const Private::FOBBVectorized& Intersection, const TRigidTransform<FReal, 3>& Transform, const QueryGeomType& QueryGeom, FReal Thickness, const FVec3& TriMeshScale, const FTriangleMeshImplicitObject* TriMesh) const
{
	FTriangleMeshOverlapVisitorNoMTD Visitor(Transform, QueryGeom, Thickness, TriMeshScale, TriMesh);
	OverlapOBB(Intersection, Visitor);

	return Visitor.bFoundIntersection;
}


TArray<int32> FTrimeshBVH::FindAllIntersections (const FAABB3& Intersection) const
{
	TArray<int32> Results;
	FTriangleMeshOverlapVisitor Visitor(Results);
	Overlap(FAABBVectorized(Intersection), Visitor);

	return Results;
}


void FTrimeshBVH::Serialize(FChaosArchive& Ar)
{
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
	Ar << Nodes;
	Ar << FaceBounds;
	if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::RemoveTriangleMeshBVHFaces)
	{
		TArray<int32> TmpFaces;
		Ar << TmpFaces;
	}
}

void FTrimeshBVH::InitializeQuantizedBounds(const TParticles<FRealSingle, 3>& Particles, const FTrimeshIndexBuffer& IndexBuffer)
{
	// Make sure we don't have the full bounds in memory if we're using quantized bounds
	FaceBounds.Empty();

	// Helper to convert vectorized AABB to a standard AABB
	auto StoreAABBVectorized = [](const FAABBVectorized& VecBounds) -> TAABB<FRealSingle, 3>
	{
		FVector3f StoreMin, StoreMax;
		VectorStoreFloat3(VecBounds.GetMin(), &StoreMin.X);
		VectorStoreFloat3(VecBounds.GetMax(), &StoreMax.X);
		return TAABB<FRealSingle, 3>(StoreMin, StoreMax);
	};

	auto GetTriBounds = [&Particles]<typename IndexBufferType>(const IndexBufferType& Buffer, int32 Idx)
	{
		FAABB3f Bounds;
		Bounds.GrowToInclude(Particles.GetX(Buffer[Idx].X));
		Bounds.GrowToInclude(Particles.GetX(Buffer[Idx].Y));
		Bounds.GrowToInclude(Particles.GetX(Buffer[Idx].Z));
		return Bounds;
	};
	
	QuantizedFaceBoundData.SetNumUninitialized(IndexBuffer.GetNumTriangles() * 6 + 1);
	for (const FNode& Node : Nodes)
	{
		for (int32 ChildIndex = 0; ChildIndex < 2; ++ChildIndex)
		{
			const FChildData& ChildData = Node.Children;
			const int32 FaceIndex = ChildData.GetChildOrFaceIndex(ChildIndex);
			if (FaceIndex != INDEX_NONE)
			{
				const uint32 FaceCount = ChildData.GetFaceCount(ChildIndex);
				if (FaceCount > 0)
				{
					TAABB<FRealSingle, 3> NodeBounds = StoreAABBVectorized(ChildData.GetBounds(ChildIndex));
					uint8* QuantizedBounds = &QuantizedFaceBoundData[FaceIndex * 6];
					for (uint32 Idx = FaceIndex, End = FaceIndex + FaceCount; Idx < End; ++Idx, QuantizedBounds += 6)
					{
						FAABB3f TriBounds = IndexBuffer.RequiresLargeIndices() ?
							GetTriBounds(IndexBuffer.GetLargeIndexBuffer(), Idx) :
							GetTriBounds(IndexBuffer.GetSmallIndexBuffer(), Idx);
						SetQuantizedBounds(QuantizedBounds, NodeBounds, TriBounds);
					}
				}
			}
		}
	}
}

bool FTrimeshBVH::NeedsQuantizedBounds()
{
	return !Nodes.IsEmpty() && FaceBounds.IsEmpty();
}

void FTrimeshBVH::ResetFaceBounds(int32 NumTris, bool bUseQuantized)
{
	if (bUseQuantized)
	{
		FaceBounds.Empty();
		QuantizedFaceBoundData.Reset(NumTris * 6 + 1);
	}
	else
	{
		QuantizedFaceBoundData.Empty();
		FaceBounds.Reset(NumTris);
	}
}

void FTrimeshBVH::AddQuantizedFaceBound(const TAABB<FRealSingle, 3>& NodeBounds, const TAABB<FRealSingle, 3>& TriBounds)
{
	QuantizedFaceBoundData.AddUninitialized(6 + (int32)(QuantizedFaceBoundData.IsEmpty()));
	// Note we always want to leave the last byte for padding, so we use Last(6) instead of Last(5) here
	SetQuantizedBounds(&QuantizedFaceBoundData.Last(6), NodeBounds, TriBounds);
}

void FTrimeshBVH::SetQuantizedBounds(uint8* QuantizedData, const TAABB<FRealSingle, 3>& NodeBounds, const TAABB<FRealSingle, 3>& TriBounds)
{
	for (int32 Idx = 0; Idx < 3; ++Idx)
	{
		FRealSingle NodeBoundsRange = NodeBounds.Max()[Idx] - NodeBounds.Min()[Idx];
		if (NodeBoundsRange < UE_SMALL_NUMBER)
		{
			QuantizedData[Idx] = 0;
			QuantizedData[Idx + 3] = 255;
		}
		else
		{
			QuantizedData[Idx] = static_cast<uint8>(FMath::Clamp(FMath::FloorToInt(255.f * (TriBounds.Min()[Idx] - NodeBounds.Min()[Idx]) / NodeBoundsRange), 0, 255));
			QuantizedData[Idx + 3] = static_cast<uint8>(FMath::Clamp(FMath::CeilToInt(255.f * (TriBounds.Max()[Idx] - NodeBounds.Min()[Idx]) / NodeBoundsRange), 0, 255));
		}
	}
}

FReal FTriangleMeshImplicitObject::PhiWithNormal(const FVec3& x, FVec3& Normal) const
{
	FSphere TestSphere(x, 0.0f);
	FRigidTransform3 TestXf(FVec3(0.0), FRotation3::FromIdentity());
	FVec3 TestLocation = x;
	FReal Depth = TNumericLimits<FReal>::Max();
	int32 FaceIndex = INDEX_NONE;
	GJKContactPointImp(TestSphere, TestXf, 0.0f, TestLocation, Normal, Depth, FaceIndex);
	return Depth;
}

template <typename IdxType>
bool FTriangleMeshImplicitObject::RaycastImp(const TArray<TVector<IdxType, 3>>& Elements, const FVec3& StartPoint, const FVec3& Dir, const FReal Length, const FReal Thickness, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex) const
{
	FTriangleMeshRaycastVisitor<IdxType> SQVisitor(StartPoint, Dir, Thickness, MParticles, Elements, bool(MeshSettingsFlags & ETriangleMeshSettingsFlags::CullsBackFaceRaycast));

	if (Thickness > 0)
	{
		FVec3 QueryHalfExtents = FVec3(Thickness) * 0.5f;
		FastBVH.Sweep(StartPoint, Dir, Length, QueryHalfExtents, SQVisitor);
	}
	else
	{
		FastBVH.Raycast(StartPoint, Dir, Length, SQVisitor);
	}

	if (SQVisitor.OutTime <= Length)
	{
		OutTime = SQVisitor.OutTime;
		OutPosition = SQVisitor.OutPosition;
		OutNormal = SQVisitor.OutNormal;
		OutFaceIndex = SQVisitor.OutFaceIndex;
		return true;
	}
	else
	{
		return false;
	}
}

bool FTriangleMeshImplicitObject::Raycast(const FVec3& StartPoint, const FVec3& Dir, const FReal Length, const FReal Thickness, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex) const
{
	if (MElements.RequiresLargeIndices())
	{
		return RaycastImp(MElements.GetLargeIndexBuffer(), StartPoint, Dir, Length, Thickness, OutTime, OutPosition, OutNormal, OutFaceIndex);
	}
	else
	{
		return RaycastImp(MElements.GetSmallIndexBuffer(), StartPoint, Dir, Length, Thickness, OutTime, OutPosition, OutNormal, OutFaceIndex);
	}
}

void FTriangleMeshImplicitObject::CollectTriangles(const FAABB3& MeshQueryBounds, const FRigidTransform3& MeshToObjectTransform, const FAABB3& ObjectBounds, Private::FMeshContactGenerator& Collector) const
{
	TArray<int32> OverlapIndices;
	FindOverlappingTriangles(MeshQueryBounds, OverlapIndices);

	const bool bStandardWinding = ((MeshToObjectTransform.GetScale3D().X * MeshToObjectTransform.GetScale3D().Y * MeshToObjectTransform.GetScale3D().Z) >= FReal(0));

	Collector.BeginCollect(OverlapIndices.Num());

	for (int32 OverlapIndex = 0; OverlapIndex < OverlapIndices.Num(); ++OverlapIndex)
	{
		const int32 TriangleIndex = OverlapIndices[OverlapIndex];
		FTriangle Triangle;
		int32 VertexIndex0, VertexIndex1, VertexIndex2;
		GetTransformedTriangle(TriangleIndex, MeshToObjectTransform, Triangle, VertexIndex0, VertexIndex1, VertexIndex2);

		if (!bStandardWinding)
		{
			Triangle.ReverseWinding();
			Swap(VertexIndex1, VertexIndex2);
		}

		const FAABB3 TriBounds = FAABB3::FromPoints(Triangle.GetVertex(0), Triangle.GetVertex(1), Triangle.GetVertex(2));
		if (TriBounds.Intersects(ObjectBounds))
		{
			Collector.AddTriangle(Triangle, TriangleIndex, VertexIndex0, VertexIndex1, VertexIndex2);
		}
	}

	Collector.EndCollect();
}

const FTrimeshBVH& FTriangleMeshImplicitObject::GetBVH() const
{
	return FastBVH;
}

template <typename QueryGeomType>
bool FTriangleMeshImplicitObject::GJKContactPointImp(const QueryGeomType& QueryGeom, const FRigidTransform3& QueryTM, const FReal WorldThickness, FVec3& Location, FVec3& Normal, FReal& OutContactPhi, int32& OutFaceIndex, FVec3 TriMeshScale) const
{
	ensure(TriMeshScale != FVec3(0.0f));
	bool bResult = false;

	const QueryGeomType& WorldScaleGeom = ScaleGeomIntoWorldHelper(QueryGeom, TriMeshScale);
	const FVec3 InvTriMeshScale = FVec3(FReal(1) / TriMeshScale.X, FReal(1) / TriMeshScale.Y, FReal(1) / TriMeshScale.Z);

	// IMPORTANT QueryTM comes with a invscaled translation so we need a version of the TM with world space translation to properly compute the bounds
	FRigidTransform3 TriMeshToGeomNoScale{ QueryTM };
	TriMeshToGeomNoScale.SetTranslation(TriMeshToGeomNoScale.GetTranslation() * TriMeshScale);
	// NOTE: BVH test is done in tri-mesh local space (whereas collision detection is done in world space because you can't non-uniformly scale all shapes)
	FAABB3 QueryBounds = WorldScaleGeom.CalculateTransformedBounds(TriMeshToGeomNoScale);
	QueryBounds.ThickenSymmetrically(FVec3(WorldThickness));
	QueryBounds.ScaleWithNegative(InvTriMeshScale);

	TRigidTransform<FReal, 3> WorldScaleQueryTM;
	ScaleTransformHelper(TriMeshScale, QueryTM, WorldScaleQueryTM);

	const bool bStandardWinding = ((TriMeshScale.X * TriMeshScale.Y * TriMeshScale.Z) >= FReal(0));

	auto CalculateTriangleContact = [&](const FVec3& A, const FVec3& B, const FVec3& C,
		FVec3& LocalContactLocation, FVec3& LocalContactNormal, FReal& LocalContactPhi) -> bool
	{
		const FVec3 AB = B - A;
		const FVec3 AC = C - A;
		FTriangle TriangleConvex(A, B, C);

		FReal LambdaPenetration;
		FVec3 ClosestA, ClosestB, LambdaNormal;
		int32 ClosestVertexIndexA, ClosestVertexIndexB;
		bool GJKValidResult = GJKPenetration<true>(TriangleConvex, WorldScaleGeom, WorldScaleQueryTM, LambdaPenetration, ClosestA, ClosestB, LambdaNormal, ClosestVertexIndexA, ClosestVertexIndexB, (FReal)0);
		if (GJKValidResult)
		{
			// @todo(chaos): single-sided collision
			LocalContactLocation = ClosestB;
			LocalContactNormal = LambdaNormal;
			LocalContactPhi = -LambdaPenetration;
		}
		return GJKValidResult;
	};


	auto LambdaHelper = [&](const auto& Elements)
	{
		FReal LocalContactPhi = FLT_MAX;
		FVec3 LocalContactLocation, LocalContactNormal;

		const TArray<int32> PotentialIntersections = FastBVH.FindAllIntersections(QueryBounds);

		for (int32 TriIdx : PotentialIntersections)
		{
			FVec3 A, B, C;
			TriangleMeshTransformVertsHelper(TriMeshScale, TriIdx, MParticles, Elements, A, B, C);

			if (!bStandardWinding)
			{
				Swap(B, C);
			}

			if (CalculateTriangleContact(A, B, C, LocalContactLocation, LocalContactNormal, LocalContactPhi))
			{
				if (LocalContactPhi < OutContactPhi)
				{
					OutContactPhi = LocalContactPhi;
					Location = LocalContactLocation;
					Normal = LocalContactNormal;
					OutFaceIndex = TriIdx;
				}
			}

		}
		return OutContactPhi < WorldThickness;
	};

	if (MElements.RequiresLargeIndices())
	{
		return LambdaHelper(MElements.GetLargeIndexBuffer());
	}
	return LambdaHelper(MElements.GetSmallIndexBuffer());
}

bool FTriangleMeshImplicitObject::GJKContactPoint(const FSphere& QueryGeom, const FRigidTransform3& QueryTM, const FReal WorldThickness, FVec3& Location, FVec3& Normal, FReal& ContactPhi, int32& ContactFaceIndex) const
{
	return GJKContactPointImp(QueryGeom, QueryTM, WorldThickness, Location, Normal, ContactPhi, ContactFaceIndex);
}

bool FTriangleMeshImplicitObject::GJKContactPoint(const TBox<FReal, 3>& QueryGeom, const FRigidTransform3& QueryTM, const FReal WorldThickness, FVec3& Location, FVec3& Normal, FReal& ContactPhi, int32& ContactFaceIndex) const
{
	return GJKContactPointImp(QueryGeom, QueryTM, WorldThickness, Location, Normal, ContactPhi, ContactFaceIndex);
}

bool FTriangleMeshImplicitObject::GJKContactPoint(const FCapsule& QueryGeom, const FRigidTransform3& QueryTM, const FReal WorldThickness, FVec3& Location, FVec3& Normal, FReal& ContactPhi, int32& ContactFaceIndex) const
{
	return GJKContactPointImp(QueryGeom, QueryTM, WorldThickness, Location, Normal, ContactPhi, ContactFaceIndex);
}

bool FTriangleMeshImplicitObject::GJKContactPoint(const FConvex& QueryGeom, const FRigidTransform3& QueryTM, const FReal WorldThickness, FVec3& Location, FVec3& Normal, FReal& ContactPhi, int32& ContactFaceIndex) const
{
	return GJKContactPointImp(QueryGeom, QueryTM, WorldThickness, Location, Normal, ContactPhi, ContactFaceIndex);
}

bool FTriangleMeshImplicitObject::GJKContactPoint(const TImplicitObjectScaled< FSphere >& QueryGeom, const FRigidTransform3& QueryTM, const FReal WorldThickness, FVec3& Location, FVec3& Normal, FReal& ContactPhi, int32& ContactFaceIndex, FVec3 TriMeshScale) const
{
	return GJKContactPointImp(QueryGeom, QueryTM, WorldThickness, Location, Normal, ContactPhi, ContactFaceIndex, TriMeshScale);
}

bool FTriangleMeshImplicitObject::GJKContactPoint(const TImplicitObjectScaled< TBox<FReal, 3> >& QueryGeom, const FRigidTransform3& QueryTM, const FReal WorldThickness, FVec3& Location, FVec3& Normal, FReal& ContactPhi, int32& ContactFaceIndex, FVec3 TriMeshScale) const
{
	return GJKContactPointImp(QueryGeom, QueryTM, WorldThickness, Location, Normal, ContactPhi, ContactFaceIndex, TriMeshScale);
}

bool FTriangleMeshImplicitObject::GJKContactPoint(const TImplicitObjectScaled< FCapsule >& QueryGeom, const FRigidTransform3& QueryTM, const FReal WorldThickness, FVec3& Location, FVec3& Normal, FReal& ContactPhi, int32& ContactFaceIndex, FVec3 TriMeshScale) const
{
	return GJKContactPointImp(QueryGeom, QueryTM, WorldThickness, Location, Normal, ContactPhi, ContactFaceIndex, TriMeshScale);
}

bool FTriangleMeshImplicitObject::GJKContactPoint(const TImplicitObjectScaled< FConvex >& QueryGeom, const FRigidTransform3& QueryTM, const FReal WorldThickness, FVec3& Location, FVec3& Normal, FReal& ContactPhi, int32& ContactFaceIndex, FVec3 TriMeshScale) const
{
	return GJKContactPointImp(QueryGeom, QueryTM, WorldThickness, Location, Normal, ContactPhi, ContactFaceIndex, TriMeshScale);
}

int32 FTriangleMeshImplicitObject::GetExternalFaceIndexFromInternal(int32 InternalFaceIndex) const
{
	if (InternalFaceIndex > -1 && ExternalFaceIndexMap.Get())
	{
		if (CHAOS_ENSURE(InternalFaceIndex >= 0 && InternalFaceIndex < ExternalFaceIndexMap->Num()))
		{
			return (*ExternalFaceIndexMap)[InternalFaceIndex];
		}
	}

	return -1;
}

bool FTriangleMeshImplicitObject::GetCullsBackFaceRaycast() const
{
	return bool(MeshSettingsFlags & ETriangleMeshSettingsFlags::CullsBackFaceRaycast);
}

void FTriangleMeshImplicitObject::SetCullsBackFaceRaycast(const bool bInCullsBackFace)
{
	if (bInCullsBackFace)
	{
		MeshSettingsFlags |= ETriangleMeshSettingsFlags::CullsBackFaceRaycast;
	}
	else
	{
		MeshSettingsFlags &= ~ETriangleMeshSettingsFlags::CullsBackFaceRaycast;
	}
}

template <typename IdxType>
bool FTriangleMeshImplicitObject::OverlapImp(const TArray<TVec3<IdxType>>& Elements, const FVec3& Point, const FReal Thickness) const
{
	FAABB3 QueryBounds(Point, Point);
	QueryBounds.Thicken(Thickness);
	const TArray<int32> PotentialIntersections = FastBVH.FindAllIntersections(QueryBounds);

	constexpr FReal Epsilon = UE_SMALL_NUMBER;
	//ensure(Thickness > Epsilon);	//There's no hope for this to work unless thickness is large (really a sphere overlap test)
	//todo: turn ensure back on, off until some other bug is fixed

	for (int32 TriIdx : PotentialIntersections)
	{
		const FVec3& A = MParticles.GetX(Elements[TriIdx][0]);
		const FVec3& B = MParticles.GetX(Elements[TriIdx][1]);
		const FVec3& C = MParticles.GetX(Elements[TriIdx][2]);

		const FVec3 AB = B - A;
		const FVec3 AC = C - A;
		FVec3 Normal = FVec3::CrossProduct(AB, AC);
		const FReal NormalLength = Normal.SafeNormalize(UE_SMALL_NUMBER);
		if (!CHAOS_ENSURE(NormalLength > Epsilon))
		{
			//hitting degenerate triangle - should be fixed before we get to this stage
			continue;
		}

		const TPlane<FReal, 3> TriPlane{A, Normal};
		const FVec3 ClosestPointOnTri = FindClosestPointOnTriangle(TriPlane, A, B, C, Point);
		const FReal Distance2 = (ClosestPointOnTri - Point).SizeSquared();
		if (Distance2 <= Thickness * Thickness) //This really only has a hope in working if thickness is > 0
		{
			return true;
		}
	}
	return false;
}

bool FTriangleMeshImplicitObject::Overlap(const FVec3& Point, const FReal Thickness) const
{
	if (MElements.RequiresLargeIndices())
	{
		return OverlapImp(MElements.GetLargeIndexBuffer(), Point, Thickness);
	}
	else
	{
		return OverlapImp(MElements.GetSmallIndexBuffer(), Point, Thickness);
	}
}


template <bool IsSpherical, typename QueryGeomType>
bool FTriangleMeshImplicitObject::OverlapGeomImp(const QueryGeomType& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD, FVec3 TriMeshScale) const
{
	const FVec3 InvTriMeshScale = FVec3(FReal(1) / TriMeshScale.X, FReal(1) / TriMeshScale.Y, FReal(1) / TriMeshScale.Z);
	const QueryGeomType& WorldScaleQueryGeom = ScaleGeomIntoWorldHelper(QueryGeom, TriMeshScale);

	if (OutMTD)
	{
		// IMPORTANT QueryTM comes with a invscaled translation so we need a version of the TM with world space translation to properly compute the bounds
		FRigidTransform3 TriMeshToGeomNoScale{ QueryTM };
		TriMeshToGeomNoScale.SetTranslation(TriMeshToGeomNoScale.GetTranslation() * TriMeshScale);

		// NOTE: BVH test is done in tri-mesh local space (whereas collision detection is done in world space because you can't non-uniformly scale all shapes)
		FAABB3 QueryBounds = WorldScaleQueryGeom.CalculateTransformedBounds(TriMeshToGeomNoScale);
		QueryBounds.ThickenSymmetrically(FVec3(Thickness));
		QueryBounds.ScaleWithNegative(InvTriMeshScale);

		const TArray<int32> PotentialIntersections = FastBVH.FindAllIntersections(QueryBounds);

		OutMTD->Normal = FVec3(0.0);
		OutMTD->Penetration = TNumericLimits<FReal>::Lowest();


		TRigidTransform<FReal, 3> WorldScaleQueryTM;
		ScaleTransformHelper(TriMeshScale, QueryTM, WorldScaleQueryTM);

		auto LambdaHelper = [&](const auto& Elements, FMTDInfo* InnerMTD)
		{
			bool bOverlap = false;
			for (int32 TriIdx : PotentialIntersections)
			{
				FVec3 A, B, C;
				TriangleMeshTransformVertsHelper(TriMeshScale, TriIdx, MParticles, Elements, A, B, C);

				FVec3 TriangleNormal(0.0);
				FReal Penetration = 0.0;
				FVec3 ClosestA(0.0);
				FVec3 ClosestB(0.0);
				int32 ClosestVertexIndexA, ClosestVertexIndexB;
				if (GJKPenetration(FTriangle(A, B, C), WorldScaleQueryGeom, WorldScaleQueryTM, Penetration, ClosestA, ClosestB, TriangleNormal, ClosestVertexIndexA, ClosestVertexIndexB, Thickness))
				{
					bOverlap = true;

					// Use Deepest MTD.
					if (Penetration > InnerMTD->Penetration)
					{
						InnerMTD->Penetration = Penetration;
						InnerMTD->Normal = TriangleNormal;
						InnerMTD->Position = ClosestA;
					}
				}
			}

			return bOverlap;
		};

		if (MElements.RequiresLargeIndices())
		{
			return LambdaHelper(MElements.GetLargeIndexBuffer(), OutMTD);
		}
		else
		{
			return LambdaHelper(MElements.GetSmallIndexBuffer(), OutMTD);
		}
	}
	else
	{
		
		FRigidTransform3 TriMeshToGeomNoScale{ QueryTM };
		TriMeshToGeomNoScale.SetTranslation(TriMeshToGeomNoScale.GetTranslation() * TriMeshScale);
		// If the query shape is axis aligned, meaning has no rotation or a 90 degree rotation use a middle phase AABB query
		if (IsSpherical || FMath::Abs(QueryTM.GetRotation().W) > 0.9 || FMath::IsNearlyEqual(QueryTM.GetRotation().W, UE_HALF_SQRT_2, 0.01))
		{
			// IMPORTANT QueryTM comes with a invscaled translation so we need a version of the TM with world space translation to properly compute the bounds
			// NOTE: BVH test is done in tri-mesh local space (whereas collision detection is done in world space because you can't non-uniformly scale all shapes)
			FAABB3 QueryBounds = WorldScaleQueryGeom.CalculateTransformedBounds(TriMeshToGeomNoScale);
			QueryBounds.ThickenSymmetrically(FVec3(Thickness));
			QueryBounds.ScaleWithNegative(InvTriMeshScale);
			TRigidTransform<FReal, 3> WorldScaleQueryTM;
			ScaleTransformHelper(TriMeshScale, QueryTM, WorldScaleQueryTM);
			return FastBVH.FindAllIntersectionsNoMTD(QueryBounds, WorldScaleQueryTM, WorldScaleQueryGeom, Thickness, TriMeshScale, this);
		}
		else
		{
			FAABB3 GeometryAABB = WorldScaleQueryGeom.BoundingBox();
			GeometryAABB.ThickenSymmetrically(FVec3(Thickness));
			// The geometry is not necessarily in the center of its translation. (cf: a capsule can be create with two points away from its own transform)
			TriMeshToGeomNoScale.SetTranslation(TriMeshToGeomNoScale.TransformPositionNoScale(GeometryAABB.Center()));
			Private::FOBBVectorized QueryObb(TriMeshToGeomNoScale, (GeometryAABB.Max() - GeometryAABB.Min()) * 0.5, InvTriMeshScale);
			TRigidTransform<FReal, 3> WorldScaleQueryTM;
			ScaleTransformHelper(TriMeshScale, QueryTM, WorldScaleQueryTM);
			return FastBVH.FindAllIntersectionsNoMTD(QueryObb, WorldScaleQueryTM, WorldScaleQueryGeom, Thickness, TriMeshScale, this);
		}
	}
}

bool FTriangleMeshImplicitObject::OverlapGeom(const FSphere& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD) const
{
	return OverlapGeomImp<true>(QueryGeom, QueryTM, Thickness, OutMTD);
}

bool FTriangleMeshImplicitObject::OverlapGeom(const TBox<FReal, 3>& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD) const
{
	return OverlapGeomImp<false>(QueryGeom, QueryTM, Thickness, OutMTD);
}

bool FTriangleMeshImplicitObject::OverlapGeom(const FCapsule& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD) const
{
	return OverlapGeomImp<false>(QueryGeom, QueryTM, Thickness, OutMTD);
}

bool FTriangleMeshImplicitObject::OverlapGeom(const FConvex& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD) const
{
	return OverlapGeomImp<false>(QueryGeom, QueryTM, Thickness, OutMTD);
}

bool FTriangleMeshImplicitObject::OverlapGeom(const TImplicitObjectScaled<FSphere>& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD, FVec3 TriMeshScale) const
{
	return OverlapGeomImp<true>(QueryGeom, QueryTM, Thickness, OutMTD, TriMeshScale);
}

bool FTriangleMeshImplicitObject::OverlapGeom(const TImplicitObjectScaled<TBox<FReal, 3>>& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD, FVec3 TriMeshScale) const
{
	return OverlapGeomImp<false>(QueryGeom, QueryTM, Thickness, OutMTD, TriMeshScale);
}

bool FTriangleMeshImplicitObject::OverlapGeom(const TImplicitObjectScaled<FCapsule>& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD, FVec3 TriMeshScale) const
{
	return OverlapGeomImp<false>(QueryGeom, QueryTM, Thickness, OutMTD, TriMeshScale);
}

bool FTriangleMeshImplicitObject::OverlapGeom(const TImplicitObjectScaled<FConvex>& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD, FVec3 TriMeshScale) const
{
	return OverlapGeomImp<false>(QueryGeom, QueryTM, Thickness, OutMTD, TriMeshScale);
}

template <typename QueryGeomType, typename IdxType>
struct FTriangleMeshSweepVisitor
{
	FTriangleMeshSweepVisitor(const FTriangleMeshImplicitObject& InTriMesh, const TArray<TVec3<IdxType>>& InElements, const QueryGeomType& InQueryGeom,
		const FVec3& InScaledDirNormalized, const FReal InLengthScale, const FRigidTransform3& InScaledStartTM, const bool InComputeMTD, FVec3 InTriMeshScale, FReal InCullsBackFaceSweepsCode)
	: TriMesh(InTriMesh)
	, Elements(InElements)
	, QueryGeom(InQueryGeom)
	, bComputeMTD(InComputeMTD)
	, CullsBackFaceSweepsCode(InCullsBackFaceSweepsCode)
	, ScaledDirNormalized(InScaledDirNormalized)
	, LengthScale(static_cast<FRealSingle>(InLengthScale))
	, OutTime(TNumericLimits<FRealSingle>::Max())
	, OutFaceIndex(INDEX_NONE)
	, TriMeshScale(InTriMeshScale)
	{
		VectorScaledDirNormalized = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(InScaledDirNormalized.X, InScaledDirNormalized.Y, InScaledDirNormalized.Z, 0.0));
		VectorCullsBackFaceSweepsCode = MakeVectorRegisterFloatFromDouble(VectorLoadFloat1(&InCullsBackFaceSweepsCode));

		const UE::Math::TQuat<FReal>& RotationDouble = InScaledStartTM.GetRotation();
		RotationSimd = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(RotationDouble.X, RotationDouble.Y, RotationDouble.Z, RotationDouble.W));

		const UE::Math::TVector<FReal>& TranslationDouble = InScaledStartTM.GetTranslation();
		TranslationSimd = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(TranslationDouble.X, TranslationDouble.Y, TranslationDouble.Z, 0.0));

		// Normalize rotation
		RotationSimd = VectorNormalizeSafe(RotationSimd, GlobalVectorConstants::Float0001);
		RayDirSimd = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(ScaledDirNormalized[0], ScaledDirNormalized[1], ScaledDirNormalized[2], 0.0));

		GeometryAABBTriangleSpace = FAABBVectorized{ InQueryGeom.BoundingBox().TransformedAABB(InScaledStartTM)};
	}

	const void* GetQueryData() const { return nullptr; }
	const void* GetSimData() const { return nullptr; }

	/** Return a pointer to the payload on which we are querying the acceleration structure */
	const void* GetQueryPayload() const
	{
		return nullptr;
	}

	bool VisitOverlap(const TSpatialVisitorData<int32>& VisitData)
	{
		check(false);
		return true;
	}

	bool VisitRaycast(const TSpatialVisitorData<int32>& VisitData, FRealSingle& CurDataLength)
	{
		check(false);
		return true;
	}

	bool VisitSweep(const TSpatialVisitorData<int32>& VisitData, FRealSingle& CurDataUnscaledLength)
	{
		const int32 TriIdx = VisitData.Payload;

		const VectorRegister4Float TriMeshScaleVector = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(TriMeshScale.X, TriMeshScale.Y, TriMeshScale.Z, 0.0));

		const TParticles<FRealSingle, 3>& Particles = TriMesh.MParticles;

		const TVector<FRealSingle, 3>& AVec = Particles.GetX(Elements[TriIdx][0]);
		const TVector<FRealSingle, 3>& BVec = Particles.GetX(Elements[TriIdx][1]);
		const TVector<FRealSingle, 3>& CVec = Particles.GetX(Elements[TriIdx][2]);

		VectorRegister4Float A = MakeVectorRegister(AVec.X, AVec.Y, AVec.Z, 0.0f);
		VectorRegister4Float B = MakeVectorRegister(BVec.X, BVec.Y, BVec.Z, 0.0f);
		VectorRegister4Float C = MakeVectorRegister(CVec.X, CVec.Y, CVec.Z, 0.0f);

		A = VectorMultiply(A, TriMeshScaleVector);
		B = VectorMultiply(B, TriMeshScaleVector);
		C = VectorMultiply(C, TriMeshScaleVector);

		FTriangleRegister Tri(A, B, C);
		const VectorRegister4Float TriNormal = VectorCross(VectorSubtract(B, A), VectorSubtract(C, A));

		bool bIsParallelSweepAndCullsBackFace = false;
		if (CullsBackFaceSweepsCode != 0)
		{
			// For the parallel epsilon check to be consistent, we must normalize the triangle normal
			const VectorRegister4Float ReciprocalNormalLen = VectorReciprocalLen(TriNormal);
			const VectorRegister4Float NormalizedNormal = VectorMultiply(TriNormal, ReciprocalNormalLen);
			const VectorRegister4Float NDotD = VectorDot3(NormalizedNormal, VectorScaledDirNormalized);

			const VectorRegister4Float PositiveEpsilon = MakeVectorRegister(UE_KINDA_SMALL_NUMBER, UE_KINDA_SMALL_NUMBER, UE_KINDA_SMALL_NUMBER, UE_KINDA_SMALL_NUMBER);
			const VectorRegister4Float NegativeEpsilon = VectorNegate(PositiveEpsilon);

			// NDotD is only in the correct direction when all scale components are positive.
			// Here we correct it if we need to have changed the winding order of the triangle
			const VectorRegister4Float ScaleCorrectedNDotD = VectorMultiply(NDotD, VectorCullsBackFaceSweepsCode);

			// Now NDotD is correct we can see if we're approaching the front or back of the triangle
			const VectorRegister4Float IsAlongPositiveNormal = VectorCompareGT(ScaleCorrectedNDotD, PositiveEpsilon);
			const VectorRegister4Float IsAlongNegativeNormal = VectorCompareLT(ScaleCorrectedNDotD, NegativeEpsilon);
			const bool bIsBackFace = VectorMaskBits(IsAlongPositiveNormal) != 0;
			const bool bIsFrontFace = VectorMaskBits(IsAlongNegativeNormal) != 0;

			bIsParallelSweepAndCullsBackFace = (!bIsBackFace && !bIsFrontFace);

			// If this is a definitive back-face hit, always skip, even if there is an initial overlap.
			if (bIsBackFace)
			{
				return true;
			}
			// If this is parallel, we want to skip if there's no initial intersection. 
			// This is an early-out check to avoid calling GJK if there's no bounding volume intersection.
			if (bIsParallelSweepAndCullsBackFace)
			{
				// Don't cull the back face if there is a chance that we are initially overlapping.
				const VectorRegister4Float MinBounds = VectorMin(VectorMin(A, B), C);
				const VectorRegister4Float MaxBounds = VectorMax(VectorMax(A, B), C);
				FAABBVectorized TriangleAABB(MinBounds, MaxBounds);
				
				if (!GeometryAABBTriangleSpace.Intersects(TriangleAABB))
				{
					return true;
				}
			}
		}
		FRealSingle ScaledTime;
		const FRealSingle ScaledCurLength = LengthScale * CurDataUnscaledLength;
		bool bIsHitting = false;

		if (UNLIKELY(Tri.IsTooBigForSinglePrecision()))
		{
			VectorRegister4Double OutPositionDouble, OutNormalDouble;
			bIsHitting = GJKRaycast2ImplSimd<VectorRegister4Double>(Tri, QueryGeom, VectorRegister4Double(RotationSimd), VectorRegister4Double(TranslationSimd), VectorRegister4Double(RayDirSimd), ScaledCurLength, ScaledTime, OutPositionDouble, OutNormalDouble, bComputeMTD, GlobalVectorConstants::Double1000);
			OutPositionSimd = MakeVectorRegisterFloatFromDouble(OutPositionDouble);
			OutNormalSimd = MakeVectorRegisterFloatFromDouble(OutNormalDouble);
		}
		else
		{
			bIsHitting = GJKRaycast2ImplSimd(Tri, QueryGeom, RotationSimd, TranslationSimd, RayDirSimd, ScaledCurLength, ScaledTime, OutPositionSimd, OutNormalSimd, bComputeMTD, GlobalVectorConstants::Float1000);
		}
		if(bIsHitting)
		{
			// Parallel sweeps should still be culled (if culls back-face is enabled) if there is no initial intersection. 
			// We still want to keep a sweep if there is initial intersection.
			if (ScaledTime > 0 && bIsParallelSweepAndCullsBackFace)
			{
				return true;
			}

			if(ScaledTime < OutTime)
			{
				OutTime = ScaledTime;

				alignas(16) FRealSingle OutFloat[4];
				VectorStoreAligned(OutNormalSimd, OutFloat);
				OutNormal.X = OutFloat[0];
				OutNormal.Y = OutFloat[1];
				OutNormal.Z = OutFloat[2];

				VectorStoreAligned(OutPositionSimd, OutFloat);
				OutPosition.X = OutFloat[0];
				OutPosition.Y = OutFloat[1];
				OutPosition.Z = OutFloat[2];

				OutFaceIndex = TriIdx;
				VectorStoreFloat3(TriNormal, &OutFaceNormal);
				
				// We have an initial overlap (MTD of 0 counts). We cannot terminate early as there might be a "better" hit (one with a more negative Time).
				FRealSingle UnscaledTime = OutTime / LengthScale;
				if(UnscaledTime <= 0)
				{
					// In an ideal world, we'd update CurrentLength to 0 so as to prevent getting any non-zero TOI intersections,
					// however GJKRaycast does not allow a ray length of 0. We could switch to GJKPenetration, 
					// however there is currently no SIMD implementation. To work around this, we can set CurrentLength to a small number instead.
					// Doing this will always produce the correct result (in the edge case we get a TOI between 0 and espilon, we'll still pick the smallest TOI).
					// This might be slightly slower in this edge case as GJK might've been able to terminate earlier.
					// #TODO Replace this pathway with a call to GJKPenetration when a SIMD version is created.
					CurDataUnscaledLength = UE_KINDA_SMALL_NUMBER;
					return true;
				}

				CurDataUnscaledLength = static_cast<FRealSingle>(UnscaledTime);
			}
		}

		return true;
	}

	const FTriangleMeshImplicitObject& TriMesh;
	const TArray<TVec3<IdxType>>& Elements;
	const FRigidTransform3 StartTM;
	const QueryGeomType& QueryGeom;
	const bool bComputeMTD;
	const FReal CullsBackFaceSweepsCode; // 0: no culling, 1/-1: winding order
	VectorRegister4Float VectorCullsBackFaceSweepsCode; // 0: no culling, 1/-1: winding order

	// Cache these values for Scaled Triangle Mesh, as they are needed for transformation when sweeping against triangles.
	FVec3 ScaledDirNormalized;
	VectorRegister4Float VectorScaledDirNormalized;
	FRealSingle LengthScale;

	FRealSingle OutTime;
	FVec3 OutPosition;
	FVec3 OutNormal;
	int32 OutFaceIndex;
	FVec3 OutFaceNormal;

	FVec3 TriMeshScale;

	VectorRegister4Float RotationSimd;
	VectorRegister4Float TranslationSimd;
	VectorRegister4Float RayDirSimd;
	VectorRegister4Float OutPositionSimd, OutNormalSimd;
	FAABBVectorized		 GeometryAABBTriangleSpace;
};

template <typename QueryGeomType, typename IdxType>
struct FTriangleMeshSweepVisitorCCD
{
	FTriangleMeshSweepVisitorCCD(const FTriangleMeshImplicitObject& InTriMesh, const TArray<TVec3<IdxType>>& InElements, const QueryGeomType& InQueryGeom,
		const FVec3& InScaledDirNormalized, const FReal InLengthScale, const FReal InLength, const FRigidTransform3& InScaledStartTM, FVec3 InTriMeshScale, FReal InCullsBackFaceSweepsCode, const FReal InIgnorePenetration, const FReal InTargetPenetration)
		: OutTOI(TNumericLimits<FRealSingle>::Max())
		, OutPhi(TNumericLimits<FRealSingle>::Max())
		, OutFaceIndex(INDEX_NONE)
		, TriMesh(InTriMesh)
		, Elements(InElements)
		, QueryGeom(InQueryGeom)
		, IgnorePenetration(InIgnorePenetration)
		, TargetPenetration(InTargetPenetration)
		, CullsBackFaceSweepsCode(InCullsBackFaceSweepsCode)
		, LengthScale(FRealSingle(InLengthScale))
		, Length(FRealSingle(InLength))
		, TriMeshScale(InTriMeshScale)
	{
		VectorScaledDirNormalized = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(InScaledDirNormalized.X, InScaledDirNormalized.Y, InScaledDirNormalized.Z, 0.0));
		VectorCullsBackFaceSweepsCode = MakeVectorRegisterFloatFromDouble(VectorLoadFloat1(&InCullsBackFaceSweepsCode));

		const UE::Math::TQuat<FReal>& RotationDouble = InScaledStartTM.GetRotation();
		RotationSimd = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(RotationDouble.X, RotationDouble.Y, RotationDouble.Z, RotationDouble.W));

		const UE::Math::TVector<FReal>& TranslationDouble = InScaledStartTM.GetTranslation();
		TranslationSimd = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(TranslationDouble.X, TranslationDouble.Y, TranslationDouble.Z, 0.0));

		// Normalize rotation
		RotationSimd = VectorNormalizeSafe(RotationSimd, GlobalVectorConstants::Float0001);
		RayDirSimd = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(InScaledDirNormalized[0], InScaledDirNormalized[1], InScaledDirNormalized[2], 0.0));
	}

	const void* GetQueryData() const { return nullptr; }
	const void* GetSimData() const { return nullptr; }

	/** Return a pointer to the payload on which we are querying the acceleration structure */
	const void* GetQueryPayload() const
	{
		return nullptr;
	}

	bool VisitOverlap(const TSpatialVisitorData<int32>& VisitData)
	{
		check(false);
		return true;
	}

	bool VisitRaycast(const TSpatialVisitorData<int32>& VisitData, FRealSingle& CurDataLength)
	{
		check(false);
		return true;
	}

	bool VisitSweep(const TSpatialVisitorData<int32>& VisitData, FRealSingle& CurDataUnscaledLength)
	{
		const int32 TriIdx = VisitData.Payload;

		const VectorRegister4Float TriMeshScaleVector = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(TriMeshScale.X, TriMeshScale.Y, TriMeshScale.Z, 0.0));

		const TParticles<FRealSingle, 3>& Particles = TriMesh.MParticles;

		const TVector<FRealSingle, 3>& AVec = Particles.GetX(Elements[TriIdx][0]);
		const TVector<FRealSingle, 3>& BVec = Particles.GetX(Elements[TriIdx][1]);
		const TVector<FRealSingle, 3>& CVec = Particles.GetX(Elements[TriIdx][2]);

		VectorRegister4Float A = MakeVectorRegister(AVec.X, AVec.Y, AVec.Z, 0.0f);
		VectorRegister4Float B = MakeVectorRegister(BVec.X, BVec.Y, BVec.Z, 0.0f);
		VectorRegister4Float C = MakeVectorRegister(CVec.X, CVec.Y, CVec.Z, 0.0f);

		A = VectorMultiply(A, TriMeshScaleVector);
		B = VectorMultiply(B, TriMeshScaleVector);
		C = VectorMultiply(C, TriMeshScaleVector);

		FTriangleRegister Tri(A, B, C);
		const VectorRegister4Float TriNormal = VectorCross(VectorSubtract(B, A), VectorSubtract(C, A));

		if (CullsBackFaceSweepsCode != 0)
		{
			const VectorRegister4Float ReturnTrue = VectorCompareGT(VectorMultiply(VectorDot3(TriNormal, VectorScaledDirNormalized), VectorCullsBackFaceSweepsCode), VectorZero());
			if (VectorMaskBits(ReturnTrue))
			{
				return true;
			}
		}

		// If we are definitely penetrating by less than IgnorePenetration at T=1, we can ignore this triangle
		// Checks the query geometry support point along the normal at the sweep end point
		// NOTE: We are using SupportCore, so we need to add Radius to the early-out distance check. We could just use Support
		// but this way avoids an extra sqrt. We don't modify IgnorePenetration in the ctor because it gets used below.
		VectorRegister4Float RadiusV = VectorSetFloat1(QueryGeom.GetRadiusf());
		VectorRegister4Float IgnorePenetrationSimd = VectorSubtract(MakeVectorRegisterFloatFromDouble(VectorSetFloat1(IgnorePenetration)), RadiusV);
		VectorRegister4Float LengthSimd = MakeVectorRegisterFloatFromDouble(VectorSetFloat1(LengthScale * Length));
		VectorRegister4Float EndPointSimd = VectorAdd(TranslationSimd, VectorMultiply(RayDirSimd, LengthSimd));
		VectorRegister4Float TriangleNormalSimd = VectorNormalize(VectorCross(VectorSubtract(B, A), VectorSubtract(C, A)));
		VectorRegister4Float OtherTriangleNormalSimd = VectorQuaternionInverseRotateVector(RotationSimd, TriangleNormalSimd);
		VectorRegister4Float OtherTrianglePositionSimd = VectorQuaternionInverseRotateVector(RotationSimd, VectorSubtract(A, EndPointSimd));
		VectorRegister4Float OtherSupportSimd = QueryGeom.SupportCoreSimd(VectorNegate(OtherTriangleNormalSimd), 0);
		VectorRegister4Float MaxDepthSimd = VectorDot3(VectorSubtract(OtherTrianglePositionSimd, OtherSupportSimd), OtherTriangleNormalSimd);
		if (VectorMaskBits(VectorCompareLT(MaxDepthSimd, IgnorePenetrationSimd)))
		{
			return true;
		}

		const FRealSingle ScaledCurLength = LengthScale * CurDataUnscaledLength;
		FRealSingle ScaledDistance;
		VectorRegister4Float PositionSimd, NormalSimd;
		if (GJKRaycast2ImplSimd(Tri, QueryGeom, RotationSimd, TranslationSimd, RayDirSimd, ScaledCurLength, ScaledDistance, PositionSimd, NormalSimd, true, GlobalVectorConstants::Float1000))
		{
			const VectorRegister4Float DirDotNormalSimd = VectorDot3(RayDirSimd, NormalSimd);
			FReal DirDotNormal;
			VectorStoreFloat1(DirDotNormalSimd, &DirDotNormal);

			// Calculate the time to reach a depth of TargetPenetration
			FReal TargetTOI, TargetPhi;
			// Make sure to compute the TOI against the original length, not the current length.
			const FReal ScaledLength = Length * LengthScale;
			ComputeSweptContactTOIAndPhiAtTargetPenetration(DirDotNormal, ScaledLength, FReal(ScaledDistance), IgnorePenetration, TargetPenetration, TargetTOI, TargetPhi);

			if (ScaledDistance < ScaledCurLength)
			{
				CurDataUnscaledLength = FRealSingle(ScaledDistance) / LengthScale;
				VectorStoreFloat3(MakeVectorRegisterDouble(PositionSimd), &OutPosition);
				VectorStoreFloat3(MakeVectorRegisterDouble(NormalSimd), &OutNormal);

				OutTOI = TargetTOI;
				OutPhi = TargetPhi;
				OutFaceIndex = TriIdx;
			}
		}

		return (CurDataUnscaledLength > 0);
	}

	FReal OutTOI;
	FReal OutPhi;
	FVec3 OutPosition;
	FVec3 OutNormal;
	int32 OutFaceIndex;

private:
	const FTriangleMeshImplicitObject& TriMesh;
	const TArray<TVec3<IdxType>>& Elements;
	const QueryGeomType& QueryGeom;
	const FReal IgnorePenetration;
	const FReal TargetPenetration;
	const FReal CullsBackFaceSweepsCode; // 0: no culling, 1/-1: winding order
	const FRealSingle LengthScale;
	const FRealSingle Length;
	const FVec3 TriMeshScale;

	VectorRegister4Float RotationSimd;
	VectorRegister4Float TranslationSimd;
	VectorRegister4Float RayDirSimd;
	VectorRegister4Float VectorScaledDirNormalized;
	VectorRegister4Float VectorCullsBackFaceSweepsCode; // 0: no culling, 1/-1: winding order
};

void ComputeScaledSweepInputs(FVec3 TriMeshScale, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length,
	FVec3& OutScaledDirNormalized, FReal& OutLengthScale, FRigidTransform3& OutScaledStartTM)
{
	const FVec3 UnscaledDirDenorm = TriMeshScale * Dir;
	const FReal LengthScale = UnscaledDirDenorm.Size();
	if (CHAOS_ENSURE(LengthScale > TNumericLimits<FReal>::Min()))
	{
		const FReal LengthScaleInv = 1.f / LengthScale;
		OutScaledDirNormalized = UnscaledDirDenorm * LengthScaleInv;
	}
	else
	{
		OutScaledDirNormalized = FVec3(1.0, 0.0, 0.0);
	}


	OutLengthScale = LengthScale;
	OutScaledStartTM = FRigidTransform3(StartTM.GetLocation() * TriMeshScale, StartTM.GetRotation());
}

FVec3 SafeInvScale(const FVec3& Scale)
{
	constexpr FReal MinMagnitude = 1e-6f; // consistent with ImplicitObjectScaled::SetScale
	FVec3 InvScale;
	for (int Axis = 0; Axis < 3; ++Axis)
	{
		if (FMath::Abs(Scale[Axis]) < MinMagnitude)
		{
			InvScale[Axis] = 1 / MinMagnitude;
		}
		else
		{
			InvScale[Axis] = 1 / Scale[Axis];
		}
	}
	return InvScale;
}

template <typename QueryGeomType>
bool FTriangleMeshImplicitObject::SweepGeomImp(const QueryGeomType& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, 
	const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FReal Thickness, 
	const bool bComputeMTD, FVec3 TriMeshScale, bool bOutputIncludesScale) const
{
	//QUICK_SCOPE_CYCLE_COUNTER(TrimeshSweep);
	// Compute scaled sweep inputs to cache in visitor.
	FVec3 ScaledDirNormalized;
	FReal LengthScale;
	FRigidTransform3 ScaledStartTM;
	ComputeScaledSweepInputs(TriMeshScale, StartTM, Dir, Length, ScaledDirNormalized, LengthScale, ScaledStartTM);

	bool bHit = false;
	auto LambdaHelper = [&](const auto& Elements)
	{
		// When length is zero, the direction of the ray is currently a hardcoded value (since it's computed from the start/end which would be equal). 
		// We don't want the backface culling check to be arbitrary based upon the hardcoded direction, so we disable it with zero-length sweeps.
		const FReal CullsBackFaceRaycastCode = (bool(MeshSettingsFlags & ETriangleMeshSettingsFlags::CullsBackFaceRaycast) && Length != 0) ? GetWindingOrder(TriMeshScale) : 0.f;
		using VisitorType = FTriangleMeshSweepVisitor<QueryGeomType,decltype(Elements[0][0])>;
		VisitorType SQVisitor(*this, Elements, QueryGeom, ScaledDirNormalized, LengthScale, ScaledStartTM, bComputeMTD, TriMeshScale, CullsBackFaceRaycastCode);

		const FAABB3 QueryBounds = QueryGeom.CalculateTransformedBounds(FRigidTransform3(FVec3::ZeroVector, StartTM.GetRotation()));
		const FVec3 InvTriMeshScale = SafeInvScale(TriMeshScale);
		const FVec3 StartPoint = QueryBounds.Center() * InvTriMeshScale + StartTM.GetLocation();
		const FVec3 Inflation = QueryBounds.Extents() * InvTriMeshScale.GetAbs() * 0.5 + FVec3(Thickness);
		FastBVH.template Sweep<VisitorType>(StartPoint, Dir, Length, Inflation, SQVisitor);

		// OutTime is always in scaled space
		if (SQVisitor.OutTime <= Length * LengthScale)
		{
			TransformSweepOutputsHelper(TriMeshScale, SQVisitor.OutNormal, SQVisitor.OutPosition, LengthScale, SQVisitor.OutTime, bOutputIncludesScale, OutNormal, OutPosition, OutTime);
			OutFaceIndex = SQVisitor.OutFaceIndex;
			OutFaceNormal = GetFaceNormal(OutFaceIndex);
			bHit = true;
		}
	};

	if(MElements.RequiresLargeIndices())
	{
		LambdaHelper(MElements.GetLargeIndexBuffer());
	}
	else
	{
		LambdaHelper(MElements.GetSmallIndexBuffer());
	}
	return bHit;
}

bool FTriangleMeshImplicitObject::SweepGeom(const TSphere<FReal,3>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FReal Thickness, const bool bComputeMTD, FVec3 TriMeshScale, bool bOutputIncludesScale) const
{
	return SweepGeomImp(QueryGeom, StartTM, Dir, Length, OutTime, OutPosition, OutNormal, OutFaceIndex, OutFaceNormal, Thickness, bComputeMTD, TriMeshScale, bOutputIncludesScale);
}

bool FTriangleMeshImplicitObject::SweepGeom(const TBox<FReal, 3>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FReal Thickness, const bool bComputeMTD, FVec3 TriMeshScale, bool bOutputIncludesScale) const
{
	return SweepGeomImp(QueryGeom, StartTM, Dir, Length, OutTime, OutPosition, OutNormal, OutFaceIndex, OutFaceNormal, Thickness, bComputeMTD, TriMeshScale, bOutputIncludesScale);
}

bool FTriangleMeshImplicitObject::SweepGeom(const FCapsule& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FReal Thickness, const bool bComputeMTD, FVec3 TriMeshScale, bool bOutputIncludesScale) const
{
	return SweepGeomImp(QueryGeom, StartTM, Dir, Length, OutTime, OutPosition, OutNormal, OutFaceIndex, OutFaceNormal, Thickness, bComputeMTD, TriMeshScale, bOutputIncludesScale);
}

bool FTriangleMeshImplicitObject::SweepGeom(const FConvex& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FReal Thickness, const bool bComputeMTD, FVec3 TriMeshScale, bool bOutputIncludesScale) const
{
	return SweepGeomImp(QueryGeom, StartTM, Dir, Length, OutTime, OutPosition, OutNormal, OutFaceIndex, OutFaceNormal, Thickness, bComputeMTD, TriMeshScale, bOutputIncludesScale);
}

bool FTriangleMeshImplicitObject::SweepGeom(const TImplicitObjectScaled<FSphere>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FReal Thickness, const bool bComputeMTD, FVec3 TriMeshScale, bool bOutputIncludesScale) const
{
	return SweepGeomImp(QueryGeom, StartTM, Dir, Length, OutTime, OutPosition, OutNormal, OutFaceIndex, OutFaceNormal, Thickness, bComputeMTD, TriMeshScale, bOutputIncludesScale);
}

bool FTriangleMeshImplicitObject::SweepGeom(const TImplicitObjectScaled<TBox<FReal, 3>>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FReal Thickness, const bool bComputeMTD, FVec3 TriMeshScale, bool bOutputIncludesScale) const
{
	return SweepGeomImp(QueryGeom, StartTM, Dir, Length, OutTime, OutPosition, OutNormal, OutFaceIndex, OutFaceNormal, Thickness, bComputeMTD, TriMeshScale, bOutputIncludesScale);
}

bool FTriangleMeshImplicitObject::SweepGeom(const TImplicitObjectScaled<FCapsule>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FReal Thickness, const bool bComputeMTD, FVec3 TriMeshScale, bool bOutputIncludesScale) const
{
	return SweepGeomImp(QueryGeom, StartTM, Dir, Length, OutTime, OutPosition, OutNormal, OutFaceIndex, OutFaceNormal, Thickness, bComputeMTD, TriMeshScale, bOutputIncludesScale);
}

bool FTriangleMeshImplicitObject::SweepGeom(const TImplicitObjectScaled<FConvex>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FReal Thickness, const bool bComputeMTD, FVec3 TriMeshScale, bool bOutputIncludesScale) const
{
	return SweepGeomImp(QueryGeom, StartTM, Dir, Length, OutTime, OutPosition, OutNormal, OutFaceIndex, OutFaceNormal, Thickness, bComputeMTD, TriMeshScale, bOutputIncludesScale);
}


template<typename QueryGeomType>
bool FTriangleMeshImplicitObject::SweepGeomCCDImp(const QueryGeomType& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, const FReal InIgnorePenetration, const FReal InTargetPenetration, FReal& OutTOI, FReal& OutPhi, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FVec3& TriMeshScale, bool bOutputIncludesScale) const
{
	// Compute scaled sweep inputs to cache in visitor.
	FVec3 ScaledDirNormalized;
	FReal LengthScale;
	FRigidTransform3 ScaledStartTM;
	ComputeScaledSweepInputs(TriMeshScale, StartTM, Dir, Length, ScaledDirNormalized, LengthScale, ScaledStartTM);

	bool bHit = false;
	auto LambdaHelper = [&](const auto& Elements)
	{
		// Emulate the old TargetPenetration mode which always uses the first contact encountered, as opposed to the first contact which reaches a depth of TargetPenetration
		const FReal IgnorePenetration = CVars::bCCDNewTargetDepthMode ? InIgnorePenetration : 0;
		const FReal TargetPenetration = CVars::bCCDNewTargetDepthMode ? InTargetPenetration : 0;

		const FReal CullsBackFaceRaycastCode = bool(MeshSettingsFlags & ETriangleMeshSettingsFlags::CullsBackFaceRaycast) ? GetWindingOrder(TriMeshScale) : 0.f;
		using VisitorType = FTriangleMeshSweepVisitorCCD<QueryGeomType, decltype(Elements[0][0])>;
		VisitorType SQVisitor(*this, Elements, QueryGeom, ScaledDirNormalized, LengthScale, Length, ScaledStartTM, TriMeshScale, CullsBackFaceRaycastCode, IgnorePenetration, TargetPenetration);

		const FAABB3 QueryBounds = QueryGeom.CalculateTransformedBounds(FRigidTransform3(FVec3::ZeroVector, StartTM.GetRotation()));
		const FVec3 InvTriMeshScale = SafeInvScale(TriMeshScale);
		const FVec3 StartPoint = QueryBounds.Center() * InvTriMeshScale + StartTM.GetLocation();
		const FVec3 Inflation = QueryBounds.Extents() * InvTriMeshScale.GetAbs() * 0.5;
		FastBVH.template Sweep<VisitorType>(StartPoint, Dir, Length, Inflation, SQVisitor);

		FReal TOI = SQVisitor.OutTOI;
		FReal Phi = SQVisitor.OutPhi;

		// @todo(chaos): Legacy path to be removed when fully tested. See ComputeSweptContactTOIAndPhiAtTargetPenetration
		if (!CVars::bCCDNewTargetDepthMode)
		{
			LegacyComputeSweptContactTOIAndPhiAtTargetPenetration(FVec3::DotProduct(ScaledDirNormalized, SQVisitor.OutNormal), LengthScale * Length, InIgnorePenetration, InTargetPenetration, TOI, Phi);
		}

		if (TOI <= 1)
		{
			OutTOI = TOI;
			OutPhi = Phi;
			FReal UnusedTime = 0;
			TransformSweepOutputsHelper(TriMeshScale, SQVisitor.OutNormal, SQVisitor.OutPosition, LengthScale, UnusedTime, bOutputIncludesScale, OutNormal, OutPosition, UnusedTime);
			OutFaceIndex = SQVisitor.OutFaceIndex;
			OutFaceNormal = GetFaceNormal(OutFaceIndex);
			bHit = true;
		}
	};

	if (MElements.RequiresLargeIndices())
	{
		LambdaHelper(MElements.GetLargeIndexBuffer());
	}
	else
	{
		LambdaHelper(MElements.GetSmallIndexBuffer());
	}
	return bHit;
}

bool FTriangleMeshImplicitObject::SweepGeomCCD(const FSphere& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& OutTOI, FReal& OutPhi, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FVec3& TriMeshScale, bool bOutputIncludesScale) const
{
	return SweepGeomCCDImp(QueryGeom, StartTM, Dir, Length, IgnorePenetration, TargetPenetration, OutTOI, OutPhi, OutPosition, OutNormal, OutFaceIndex, OutFaceNormal, TriMeshScale, bOutputIncludesScale);
}

bool FTriangleMeshImplicitObject::SweepGeomCCD(const TBox<FReal, 3>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& OutTOI, FReal& OutPhi, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FVec3& TriMeshScale, bool bOutputIncludesScale) const
{
	return SweepGeomCCDImp(QueryGeom, StartTM, Dir, Length, IgnorePenetration, TargetPenetration, OutTOI, OutPhi, OutPosition, OutNormal, OutFaceIndex, OutFaceNormal, TriMeshScale, bOutputIncludesScale);
}

bool FTriangleMeshImplicitObject::SweepGeomCCD(const FCapsule& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& OutTOI, FReal& OutPhi, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FVec3& TriMeshScale, bool bOutputIncludesScale) const
{
	return SweepGeomCCDImp(QueryGeom, StartTM, Dir, Length, IgnorePenetration, TargetPenetration, OutTOI, OutPhi, OutPosition, OutNormal, OutFaceIndex, OutFaceNormal, TriMeshScale, bOutputIncludesScale);
}

bool FTriangleMeshImplicitObject::SweepGeomCCD(const FConvex& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& OutTOI, FReal& OutPhi, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FVec3& TriMeshScale, bool bOutputIncludesScale) const
{
	return SweepGeomCCDImp(QueryGeom, StartTM, Dir, Length, IgnorePenetration, TargetPenetration, OutTOI, OutPhi, OutPosition, OutNormal, OutFaceIndex, OutFaceNormal, TriMeshScale, bOutputIncludesScale);
}

bool FTriangleMeshImplicitObject::SweepGeomCCD(const TImplicitObjectScaled<FSphere>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& OutTOI, FReal& OutPhi, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FVec3& TriMeshScale, bool bOutputIncludesScale) const
{
	return SweepGeomCCDImp(QueryGeom, StartTM, Dir, Length, IgnorePenetration, TargetPenetration, OutTOI, OutPhi, OutPosition, OutNormal, OutFaceIndex, OutFaceNormal, TriMeshScale, bOutputIncludesScale);
}

bool FTriangleMeshImplicitObject::SweepGeomCCD(const TImplicitObjectScaled<TBox<FReal, 3>>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& OutTOI, FReal& OutPhi, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FVec3& TriMeshScale, bool bOutputIncludesScale) const
{
	return SweepGeomCCDImp(QueryGeom, StartTM, Dir, Length, IgnorePenetration, TargetPenetration, OutTOI, OutPhi, OutPosition, OutNormal, OutFaceIndex, OutFaceNormal, TriMeshScale, bOutputIncludesScale);
}

bool FTriangleMeshImplicitObject::SweepGeomCCD(const TImplicitObjectScaled<FCapsule>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& OutTOI, FReal& OutPhi, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FVec3& TriMeshScale, bool bOutputIncludesScale) const
{
	return SweepGeomCCDImp(QueryGeom, StartTM, Dir, Length, IgnorePenetration, TargetPenetration, OutTOI, OutPhi, OutPosition, OutNormal, OutFaceIndex, OutFaceNormal, TriMeshScale, bOutputIncludesScale);
}

bool FTriangleMeshImplicitObject::SweepGeomCCD(const TImplicitObjectScaled<FConvex>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& OutTOI, FReal& OutPhi, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FVec3& TriMeshScale, bool bOutputIncludesScale) const
{
	return SweepGeomCCDImp(QueryGeom, StartTM, Dir, Length, IgnorePenetration, TargetPenetration, OutTOI, OutPhi, OutPosition, OutNormal, OutFaceIndex, OutFaceNormal, TriMeshScale, bOutputIncludesScale);
}


template <typename IdxType>
int32 FTriangleMeshImplicitObject::FindMostOpposingFace(const TArray<TVec3<IdxType>>& Elements, const FVec3& Position, const FVec3& UnitDir, int32 HintFaceIndex, FReal SearchDist, const FVec3& Scale) const
{
	//todo: this is horribly slow, need adjacency information
	const FReal SearchDist2 = SearchDist * SearchDist;

	const FVec3 InvScale = 1 / Scale;
	const FVec3 ScaledPosition = Position * InvScale;
	const FVec3 ScaledSearchDist = SearchDist * InvScale;
	const FVec3 AbsScaledSearchDist(FMath::Abs(ScaledSearchDist[0]), FMath::Abs(ScaledSearchDist[1]), FMath::Abs(ScaledSearchDist[2]));
	const FAABB3 QueryBounds(ScaledPosition - AbsScaledSearchDist, ScaledPosition + AbsScaledSearchDist);

	const TArray<int32> PotentialIntersections = FastBVH.FindAllIntersections(QueryBounds);
	constexpr FReal Epsilon = UE_SMALL_NUMBER;

	FReal MostOpposingDot = TNumericLimits<FReal>::Max();
	int32 MostOpposingFace = HintFaceIndex;

	for (int32 TriIdx : PotentialIntersections)
	{
		const FVec3& A = MParticles.GetX(Elements[TriIdx][0]) * Scale;
		const FVec3& B = MParticles.GetX(Elements[TriIdx][1]) * Scale;
		const FVec3& C = MParticles.GetX(Elements[TriIdx][2]) * Scale;

		const FVec3 AB = B - A;
		const FVec3 AC = C - A;
		FVec3 Normal = FVec3::CrossProduct(AB, AC);
		const FReal NormalLength = Normal.SafeNormalize(UE_SMALL_NUMBER);
		if (!CHAOS_ENSURE(NormalLength > Epsilon))
		{
			//hitting degenerate triangle - should be fixed before we get to this stage
			continue;
		}

		// Check if the scale is reflective
		bool bNormalSignFlip = Scale.X * Scale.Y * Scale.Z < 0;
		if (bNormalSignFlip)
		{
			Normal = -Normal;
		}

		const TPlane<FReal, 3> TriPlane{A, Normal};
		const FVec3 ClosestPointOnTri = FindClosestPointOnTriangle(TriPlane, A, B, C, Position);
		const FReal Distance2 = (ClosestPointOnTri - Position).SizeSquared();
		if (Distance2 < SearchDist2)
		{
			const FReal Dot = FVec3::DotProduct(Normal, UnitDir);
			if (Dot < MostOpposingDot)
			{
				MostOpposingDot = Dot;
				MostOpposingFace = TriIdx;
			}
		}
	}

	return MostOpposingFace;
}

int32 FTriangleMeshImplicitObject::FindMostOpposingFace(const FVec3& Position, const FVec3& UnitDir, int32 HintFaceIndex, FReal SearchDist) const
{
	const FVec3 Scale(1.0f, 1.0f, 1.0f);
	if (MElements.RequiresLargeIndices())
	{
		return FindMostOpposingFace(MElements.GetLargeIndexBuffer(), Position, UnitDir, HintFaceIndex, SearchDist, Scale);
	}
	else
	{
		return FindMostOpposingFace(MElements.GetSmallIndexBuffer(), Position, UnitDir, HintFaceIndex, SearchDist, Scale);
	}
}

int32 FTriangleMeshImplicitObject::FindMostOpposingFaceScaled(const FVec3& Position, const FVec3& UnitDir, int32 HintFaceIndex, FReal SearchDist, const FVec3& Scale) const
{
	if (MElements.RequiresLargeIndices())
	{
		return FindMostOpposingFace(MElements.GetLargeIndexBuffer(), Position, UnitDir, HintFaceIndex, SearchDist, Scale);
	}
	else
	{
		return FindMostOpposingFace(MElements.GetSmallIndexBuffer(), Position, UnitDir, HintFaceIndex, SearchDist, Scale);
	}
}

FVec3 FTriangleMeshImplicitObject::FindGeometryOpposingNormal(const FVec3& DenormDir, int32 FaceIndex, const FVec3& OriginalNormal) const
{
	if (CHAOS_ENSURE(FaceIndex != INDEX_NONE))
	{
		return GetFaceNormal(FaceIndex);
	}
	return OriginalNormal;
}

template <typename IdxType>
FImplicitObjectPtr FTriangleMeshImplicitObject::CopySlowImpl(const TArray<TVector<IdxType, 3>>& InElements) const
{
	using namespace Chaos;
	
	TArray<ParticleVecType> XArray = MParticles.AllX();
	ParticlesType ParticlesCopy(MoveTemp(XArray));
	TArray<TVector<IdxType, 3>> ElementsCopy(InElements);
	TArray<uint16> MaterialIndicesCopy = MaterialIndices;
	TUniquePtr<TArray<int32>> ExternalFaceIndexMapCopy = nullptr;
	if (ExternalFaceIndexMap)
	{
		ExternalFaceIndexMapCopy = MakeUnique<TArray<int32>>(*ExternalFaceIndexMap.Get());
	}

	TUniquePtr<TArray<int32>> ExternalVertexIndexMapCopy = nullptr;
	if (ExternalVertexIndexMap && TriMeshPerPolySupport)
	{
		ExternalVertexIndexMapCopy = MakeUnique<TArray<int32>>(*ExternalVertexIndexMap.Get());
	}

	return FImplicitObjectPtr(new FTriangleMeshImplicitObject(MoveTemp(ParticlesCopy), MoveTemp(ElementsCopy), MoveTemp(MaterialIndicesCopy), MoveTemp(ExternalFaceIndexMapCopy), MoveTemp(ExternalVertexIndexMapCopy), GetCullsBackFaceRaycast()));
}

FImplicitObjectPtr FTriangleMeshImplicitObject::DeepCopyGeometry() const
{
	if (MElements.RequiresLargeIndices())
	{
		return CopySlowImpl(MElements.GetLargeIndexBuffer());
	}
	else
	{
		return CopySlowImpl(MElements.GetSmallIndexBuffer());
	}
}
	
FImplicitObjectPtr FTriangleMeshImplicitObject::CopyGeometry() const
{
	// For now we are using a deep copy
	// since we are using refcountptr and not uniqueptr anymore
	// we can have shallow copy and have 2 different implementations 
	return DeepCopyGeometry();
}

void FTriangleMeshImplicitObject::Serialize(FChaosArchive& Ar)
{
	FChaosArchiveScopedMemory ScopedMemory(Ar, GetTypeName());
	SerializeImp(Ar);
}

uint32 FTriangleMeshImplicitObject::GetTypeHash() const
{
	uint32 Result = MParticles.GetTypeHash();
	Result = HashCombine(Result, MLocalBoundingBox.GetTypeHash());

	auto LambdaHelper = [&](const auto& Elements)
	{
		for (TVector<int32, 3> Tri : Elements)
		{
			uint32 TriHash = HashCombine(::GetTypeHash(Tri[0]), HashCombine(::GetTypeHash(Tri[1]), ::GetTypeHash(Tri[2])));
			Result = HashCombine(Result, TriHash);
		}
	};

	if (MElements.RequiresLargeIndices())
	{
		LambdaHelper(MElements.GetLargeIndexBuffer());
	}
	else
	{
		LambdaHelper(MElements.GetSmallIndexBuffer());
	}

	return Result;
}

FVec3 FTriangleMeshImplicitObject::GetFaceNormal(const int32 FaceIdx) const
{
	if (CHAOS_ENSURE(FaceIdx != INDEX_NONE))
	{
		auto LambdaHelper = [&](const auto& Elements)
		{
			const ParticleVecType& A = MParticles.GetX(Elements[FaceIdx][0]);
			const ParticleVecType& B = MParticles.GetX(Elements[FaceIdx][1]);
			const ParticleVecType& C = MParticles.GetX(Elements[FaceIdx][2]);

			const ParticleVecType AB = B - A;
			const ParticleVecType AC = C - A;
			ParticleVecType Normal = ParticleVecType::CrossProduct(AB, AC);
			
			if(Normal.SafeNormalize(UE_SMALL_NUMBER) < UE_SMALL_NUMBER)
			{
				UE_LOGF(LogChaos, Warning, "Degenerate triangle %d: (%f %f %f) (%f %f %f) (%f %f %f)", FaceIdx, A.X, A.Y, A.Z, B.X, B.Y, B.Z, C.X, C.Y, C.Z);
				CHAOS_ENSURE(false);
				return FVec3(0, 0, 1);
			}

			return FVec3(Normal);
		};
		
		if (MElements.RequiresLargeIndices())
		{
			return LambdaHelper(MElements.GetLargeIndexBuffer());
		}
		else
		{
			return LambdaHelper(MElements.GetSmallIndexBuffer());
		}
	}

	return FVec3(0, 0, 1);
}

uint16 FTriangleMeshImplicitObject::GetMaterialIndex(uint32 HintIndex) const
{
	if (MaterialIndices.IsValidIndex(HintIndex))
	{
		return MaterialIndices[HintIndex];
	}

	// 0 should always be the default material for a shape
	return 0;
}

const FTriangleMeshImplicitObject::ParticlesType& FTriangleMeshImplicitObject::Particles() const
{
	return MParticles;
}

const FTrimeshIndexBuffer& FTriangleMeshImplicitObject::Elements() const
{
	return MElements;
}

template <typename IdxType>
void FTriangleMeshImplicitObject::RebuildBVImp(const TArray<TVec3<IdxType>>& Elements, BVHType& TreeBVH)
{
	const int32 NumTris = Elements.Num();
	TArray<FBvEntry<sizeof(IdxType) == sizeof(FTrimeshIndexBuffer::LargeIdxType)>> BVEntries;
	BVEntries.Reset(NumTris);

	for (int Tri = 0; Tri < NumTris; Tri++)
	{
		BVEntries.Add({this, Tri});
	}

	// Override some parameters for the internal triangle mesh acceleration
	//
	// Children in leaf - update to tested value with higher perf for this application
	// Tree depth - default for the tree type
	// Max element bounds - as we use a faster tree derived from this that has no global item support
	//						all elements must be fully placed in the tree to be picked up when we translate
	//						them over to the fast bvh
	constexpr static int32 MaxChildrenInLeaf = 22;
	constexpr static int32 MaxTreeDepth = BVHType::DefaultMaxTreeDepth;
	constexpr static Chaos::FRealSingle MaxElementBounds = std::numeric_limits<Chaos::FRealSingle>::max();

	TreeBVH.Reinitialize(BVEntries, MaxChildrenInLeaf, MaxTreeDepth, MaxElementBounds);
}

FTriangleMeshImplicitObject::~FTriangleMeshImplicitObject() = default;


FTriangleMeshImplicitObject::ETriangleMeshSettingsFlags FTriangleMeshImplicitObject::PackSettingsFlags(bool bInCullsBackFaceRaycast, FTriangleMeshImplicitObject::FBuildSettings BuildSettings)
{
	ETriangleMeshSettingsFlags Flags{ 0 };
	if (bInCullsBackFaceRaycast)
	{
		Flags |= ETriangleMeshSettingsFlags::CullsBackFaceRaycast;
	}
	if (BuildSettings.PerFaceBounds == FTrimeshBVH::EPerFaceBoundsRepresentation::Byte)
	{
		Flags |= ETriangleMeshSettingsFlags::QuantizePerFaceBounds;
	}
	return Flags;
}


void FTriangleMeshImplicitObject::RebuildFastBVH()
{
	BVHType TreeBVH;
	if (MElements.RequiresLargeIndices())
	{
		RebuildBVImp(MElements.GetLargeIndexBuffer(), TreeBVH);
	}
	else
	{
		RebuildBVImp(MElements.GetSmallIndexBuffer(), TreeBVH);
	}
	RebuildFastBVHFromTree(TreeBVH);
}

void FTriangleMeshImplicitObject::UpdateVertices(const TArray<FVector>& NewPositions)
{
	if(TriMeshPerPolySupport == false)
	{
		// We don't have vertex map, this will not be correct.
		ensure(false);
		return;
	}

	const bool bRemapIndices = ExternalVertexIndexMap != nullptr;

	for (int32 i = 0; i < NewPositions.Num(); ++i)
	{
		int32 InternalIdx = bRemapIndices ? (*ExternalVertexIndexMap.Get())[i] : i;
		if (InternalIdx < (int32)MParticles.Size())
		{
			MParticles.SetX(InternalIdx, Chaos::FVec3(NewPositions[i]));
		}
	}
	RebuildFastBVH();
}

namespace
{
	void AddTriangles(TArray<TVec3<FTrimeshIndexBuffer::LargeIdxType>>& LargeIndices, TArray<TVec3<FTrimeshIndexBuffer::SmallIdxType>>& SmallIndices, const FTrimeshIndexBuffer& Elements, int32 OldIndex)
	{
		if (Elements.RequiresLargeIndices())
		{
			FTrimeshIndexBuffer::LargeIdxType A = Elements.GetLargeIndexBuffer()[OldIndex][0];
			FTrimeshIndexBuffer::LargeIdxType B = Elements.GetLargeIndexBuffer()[OldIndex][1];
			FTrimeshIndexBuffer::LargeIdxType C = Elements.GetLargeIndexBuffer()[OldIndex][2];

			LargeIndices.Add(TVec3<FTrimeshIndexBuffer::LargeIdxType>(A, B, C));
		}
		else
		{
			FTrimeshIndexBuffer::SmallIdxType A = Elements.GetSmallIndexBuffer()[OldIndex][0];
			FTrimeshIndexBuffer::SmallIdxType B = Elements.GetSmallIndexBuffer()[OldIndex][1];
			FTrimeshIndexBuffer::SmallIdxType C = Elements.GetSmallIndexBuffer()[OldIndex][2];

			SmallIndices.Add(TVec3<FTrimeshIndexBuffer::SmallIdxType>(A, B, C));
		}
	}
}

void FTriangleMeshImplicitObject::RebuildFastBVHFromTree(const BVHType& TreeBVH)
{
	using NodeType = TAABBTreeNode<FRealSingle>;
	using LeafType = TAABBTreeLeafArray<int32, /*bComputeBounds=*/ false, FRealSingle>;
	const TArray<NodeType>& Nodes = TreeBVH.GetNodes();
	const TArray<LeafType>& Leaves = TreeBVH.GetLeaves();

	FastBVH.Nodes.Reset();
	const bool bUseQuantizedBounds = bool(MeshSettingsFlags & ETriangleMeshSettingsFlags::QuantizePerFaceBounds);
	FastBVH.ResetFaceBounds(MElements.GetNumTriangles(), bUseQuantizedBounds);

	int32 FaceNum = 0;
	TArray<TVec3<FTrimeshIndexBuffer::LargeIdxType>> LargeIndices;
	TArray<TVec3<FTrimeshIndexBuffer::SmallIdxType>> SmallIndices;
	
	TArray<uint16> OldMaterialIndices;
	TArray<int32> OldFaceIndexMap;

	if (ExternalFaceIndexMap.IsValid())
	{
		OldFaceIndexMap = MoveTemp(*ExternalFaceIndexMap.Get());
		ExternalFaceIndexMap->Reserve(OldFaceIndexMap.Num());
	}

	OldMaterialIndices = MoveTemp(MaterialIndices);
	MaterialIndices.Reserve(OldMaterialIndices.Num());

	// since we do skip leaf nodes, we need to handle the case where we have only one node that will be a leaf by default
	if (Nodes.Num() == 1)
	{
		const NodeType& RootNode = Nodes[0];
		ensure(RootNode.bLeaf);

		// the leaf index is stored in the first  ChildrenNodes for leaf type nodes
		const int32 LeafIndex = RootNode.ChildrenNodes[0];
		const LeafType& Leaf = Leaves[LeafIndex];

		if (Leaf.Elems.Num() == 0)
		{
			// Will have an empty BVH in this case
			return;
		}

		// make the node
		FTrimeshBVH::FNode& NewNode = FastBVH.Nodes.Emplace_GetRef();
		NewNode.Children.SetChildOrFaceIndex(0, 0);
		NewNode.Children.SetFaceCount(0, Leaf.Elems.Num());
		NewNode.Children.SetBounds(0, BoundingBox());
		for (const TPayloadBoundsElement<int32, FRealSingle>& LeafPayload: Leaf.Elems)
		{
			if (bUseQuantizedBounds)
			{
				FastBVH.AddQuantizedFaceBound(BoundingBoxf(), LeafPayload.BoundingBox());
			}
			else
			{
				FastBVH.FaceBounds.Add(FAABBVectorized(LeafPayload.Bounds));
			}
			// Reorder triangle indices, triangles will be in the same order as the bounding volume in the BVH structure.
			// And all triangles in a leaf will be contiguous in memory.
			AddTriangles(LargeIndices, SmallIndices, MElements, LeafPayload.Payload);
			if (ExternalFaceIndexMap.IsValid() && !OldFaceIndexMap.IsEmpty())
			{
				ExternalFaceIndexMap->Add(OldFaceIndexMap[LeafPayload.Payload]);
			}

			// If we have materials - add the index to the remapped list
			if(OldMaterialIndices.IsValidIndex(LeafPayload.Payload))
			{
				MaterialIndices.Add(OldMaterialIndices[LeafPayload.Payload]);
			}
		}
		if (MElements.RequiresLargeIndices())
		{
			MElements.Reinitialize(MoveTemp(LargeIndices));
		}
		else
		{
			MElements.Reinitialize(MoveTemp(SmallIndices));
		}
		return;
	}
	
	// map leaf type nodes index (in Nodes array) to leaf index (in Leaves array) 
	TMap<int32, int32> ChildIndexToLeafIndexMap;
	for (int32 NodeIndex = 0; NodeIndex < Nodes.Num(); ++NodeIndex)
	{
		const NodeType& Node = Nodes[NodeIndex];
		if (Node.bLeaf)
		{
			// the leaf index is stored in the first  ChildrenNodes for leaf type nodes
			const int32 LeafIndex =  Node.ChildrenNodes[0];
			ensure(Leaves.IsValidIndex(LeafIndex));
			ChildIndexToLeafIndexMap.Emplace(NodeIndex, LeafIndex);
		}
	}

	// let's walk the tree and build the optimized data
	TArray<int32> NodeIndexStack;
	NodeIndexStack.Push(0);

	// used to remap indices at the end of the process
	TMap<int32, int32> BVHToFastBVHNodeIndexMap;
	
	while (NodeIndexStack.Num())
	{
		const int32 NodeIndex = NodeIndexStack.Pop();
		const NodeType& Node = Nodes[NodeIndex];

		// leaf nodes are being trimmed and their info will be compacted in parents
		if (!Node.bLeaf)
		{
			BVHToFastBVHNodeIndexMap.Emplace(NodeIndex, FastBVH.Nodes.Num());
			FTrimeshBVH::FNode& NewNode = FastBVH.Nodes.Emplace_GetRef();

			// go backward to simulate a depth first walk to keep the same order of the original tree 
			// for consistent results to return the same triangle when a query hit right in on a share vertex
			for (int32 ChildIndex=1; ChildIndex>=0; --ChildIndex)
			{
				// common infos
				FTrimeshBVH::FChildData& ChildData = NewNode.Children;
				ChildData.SetBounds(ChildIndex, Node.ChildrenBounds[ChildIndex]);
				const int32 ChildNodeIndex = Node.ChildrenNodes[ChildIndex];
				// index in the original BVH space, remapping is done at the end of the process
				// this may be overwritten if the child is a leaf
				ChildData.SetChildOrFaceIndex(ChildIndex, ChildNodeIndex);

				if (Nodes.IsValidIndex(ChildNodeIndex))
				{
					// let's pull the face (leaves) data for leaf child nodes
					const int32* LeafIndex = ChildIndexToLeafIndexMap.Find(ChildNodeIndex);
					if (LeafIndex)
					{
						const LeafType& Leaf = Leaves[*LeafIndex];

						// store face range in the node 
						ChildData.SetChildOrFaceIndex(ChildIndex, FaceNum);
						ChildData.SetFaceCount(ChildIndex, Leaf.Elems.Num());
						check(ChildData.GetFaceCount(ChildIndex) > 0);

						TMap<int32, int32> VertexReuse;
						// copy indices in the linear face array
						for (const auto& LeafPayload: Leaf.Elems)
						{
							if (bUseQuantizedBounds)
							{
								FastBVH.AddQuantizedFaceBound(Node.ChildrenBounds[ChildIndex], LeafPayload.Bounds);
							}
							else
							{
								FastBVH.FaceBounds.Add(FAABBVectorized(LeafPayload.Bounds));
							}
							// Reorder triangle indices, triangles will be in the same order as the bounding volume in the BVH structure.
							// And all triangles in a leaf will be contiguous in memory.
							AddTriangles(LargeIndices, SmallIndices, MElements, LeafPayload.Payload);
							if (ExternalFaceIndexMap.IsValid() && !OldFaceIndexMap.IsEmpty())
							{
								ExternalFaceIndexMap->Add(OldFaceIndexMap[LeafPayload.Payload]);
							}

							// If we have materials - add the index to the remapped list
							if(OldMaterialIndices.IsValidIndex(LeafPayload.Payload))
							{
								MaterialIndices.Add(OldMaterialIndices[LeafPayload.Payload]);
							}

							FaceNum++;
						}
					}
					// push for further processing
					NodeIndexStack.Push(ChildNodeIndex);
				}
			}
		}
	}

	// remap child node indices from original BVH node array space to fast BVH node array space
	for (int32 NodeIndex = 0; NodeIndex < FastBVH.Nodes.Num(); ++NodeIndex)
	{
		FTrimeshBVH::FNode& Node = FastBVH.Nodes[NodeIndex];
		for (int32 ChildIndex=0; ChildIndex<2; ++ChildIndex)
		{
			FTrimeshBVH::FChildData& ChildData = Node.Children;
			const bool bHasFaces = ChildData.GetFaceCount(ChildIndex) > 0;
			if (!bHasFaces)
			{
				const int32 ChildNodeIndex = ChildData.GetChildOrFaceIndex(ChildIndex);
				const int32* FixedChildNodeIndex = BVHToFastBVHNodeIndexMap.Find(ChildNodeIndex);
				if (ensure(FixedChildNodeIndex))
				{
					ChildData.SetChildOrFaceIndex(ChildIndex, *FixedChildNodeIndex);
				}
			}
		}
	}
	if (MElements.RequiresLargeIndices())
	{
		MElements.Reinitialize(MoveTemp(LargeIndices));
	}
	else
	{
		MElements.Reinitialize(MoveTemp(SmallIndices));
	}
}
	
}
