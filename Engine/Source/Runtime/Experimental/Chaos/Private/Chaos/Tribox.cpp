// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/Tribox.h"

namespace Chaos
{
namespace Private
{

// Math utilities
static constexpr FTribox::FRealType Sqrt2 = 1.414213562373095f;
static constexpr FTribox::FRealType InvSqrt2 = 1.0f / Sqrt2;

// k-DOP18 (k-Discrete Oriented Polytopes) Direction Vectors
static const FTribox::FVec3Type PlanesDirs[18] = 
{
	FTribox::FVec3Type( 1.f, 0.f, 0.f),			// Face 0
	FTribox::FVec3Type(-1.f, 0.f, 0.f),			// Face 1
	FTribox::FVec3Type( 0.f, 1.f, 0.f),			// Face 2
	FTribox::FVec3Type( 0.f,-1.f, 0.f),			// Face 3
	FTribox::FVec3Type( 0.f, 0.f, 1.f),			// Face 4
	FTribox::FVec3Type( 0.f, 0.f,-1.f),			// Face 5
	FTribox::FVec3Type( 0.f, InvSqrt2,  InvSqrt2),	// Face 6
	FTribox::FVec3Type( 0.f,-InvSqrt2, -InvSqrt2),	// Face 7
	FTribox::FVec3Type( 0.f, InvSqrt2, -InvSqrt2),	// Face 8
	FTribox::FVec3Type( 0.f,-InvSqrt2,  InvSqrt2),	// Face 9
	FTribox::FVec3Type( InvSqrt2, 0.f,  InvSqrt2),	// Face 10
	FTribox::FVec3Type(-InvSqrt2, 0.f, -InvSqrt2),	// Face 11
	FTribox::FVec3Type( InvSqrt2, 0.f, -InvSqrt2),	// Face 12
	FTribox::FVec3Type(-InvSqrt2, 0.f,  InvSqrt2),	// Face 13
	FTribox::FVec3Type( InvSqrt2,  InvSqrt2, 0.f),	// Face 14
	FTribox::FVec3Type(-InvSqrt2, -InvSqrt2, 0.f),	// Face 15
	FTribox::FVec3Type( InvSqrt2, -InvSqrt2, 0.f),	// Face 16
	FTribox::FVec3Type(-InvSqrt2,  InvSqrt2, 0.f)	// Face 17
};

// List of principal planes indices linked to each of the 8 box corner
static const int32 PrincipalIndices[8][3] =
{
	{0,2,4},
	{0,2,5},
	{0,3,4},
	{1,2,4},
	{1,3,4},
	{1,2,5},
	{0,3,5},
	{1,3,5}
};

// List of chamfer planes indices linked to each of the 8 box corner
static const int32 ChamferIndices[8][3]=
{
	{6,10,14},
	{8,12,14},
	{9,10,16},
	{6,13,17},
	{9,13,15},
	{8,11,17},
	{7,12,16},
	{7,11,15}
};

// Intersection matrix in between the 3 chamfer planes 
static const FTribox::FMatrix33Type ChamferMatrices[8] =
{
	{-InvSqrt2, InvSqrt2, InvSqrt2, InvSqrt2, -InvSqrt2, InvSqrt2, InvSqrt2, InvSqrt2, -InvSqrt2},
	{-InvSqrt2, InvSqrt2, InvSqrt2, InvSqrt2, -InvSqrt2, InvSqrt2, -InvSqrt2, -InvSqrt2, InvSqrt2}, 
	{-InvSqrt2, InvSqrt2, InvSqrt2, -InvSqrt2, InvSqrt2, -InvSqrt2, InvSqrt2, InvSqrt2, -InvSqrt2}, 
	{InvSqrt2, -InvSqrt2, -InvSqrt2, InvSqrt2, -InvSqrt2, InvSqrt2, InvSqrt2, InvSqrt2, -InvSqrt2}, 
	{InvSqrt2, -InvSqrt2, -InvSqrt2, -InvSqrt2, InvSqrt2, -InvSqrt2, InvSqrt2, InvSqrt2, -InvSqrt2}, 
	{InvSqrt2, -InvSqrt2, -InvSqrt2, InvSqrt2, -InvSqrt2, InvSqrt2, -InvSqrt2, -InvSqrt2, InvSqrt2}, 
	{-InvSqrt2, InvSqrt2, InvSqrt2, -InvSqrt2, InvSqrt2, -InvSqrt2, -InvSqrt2, -InvSqrt2, InvSqrt2}, 
	{InvSqrt2, -InvSqrt2, -InvSqrt2, -InvSqrt2, InvSqrt2, -InvSqrt2, -InvSqrt2, -InvSqrt2, InvSqrt2}
};

// Intersection matrix in between the 2 chamfer planes and a principal one
// (9 combinations for the choice of the 2 chamfer + 1 principal)
static const FTribox::FMatrix33Type PrincipalMatrices[8][9]=
{
	{
		{1.000000, 0.000000, -0.000000, -1.000000, -0.000000, Sqrt2, -1.000000, Sqrt2, -0.000000},
		{0.000000, 1.000000, 0.000000, -0.000000, -1.000000, Sqrt2, Sqrt2, 1.000000, -Sqrt2},
		{0.000000, -0.000000, 1.000000, Sqrt2, -Sqrt2, 1.000000, 0.000000, Sqrt2, -1.000000},
		{-1.000000, -0.000000, Sqrt2, 1.000000, 0.000000, -0.000000, 1.000000, Sqrt2, -Sqrt2},
		{-0.000000, -1.000000, Sqrt2, 0.000000, 1.000000, 0.000000, Sqrt2, -1.000000, -0.000000},
		{-Sqrt2, Sqrt2, 1.000000, -0.000000, 0.000000, 1.000000, Sqrt2, -0.000000, -1.000000},
		{-1.000000, Sqrt2, 0.000000, 1.000000, -Sqrt2, Sqrt2, 1.000000, -0.000000, 0.000000},
		{-Sqrt2, 1.000000, Sqrt2, Sqrt2, -1.000000, -0.000000, 0.000000, 1.000000, 0.000000},
		{-0.000000, Sqrt2, -1.000000, Sqrt2, -0.000000, -1.000000, -0.000000, 0.000000, 1.000000}
	},
	{
		{1.000000, -0.000000, 0.000000, -1.000000, 0.000000, Sqrt2, 1.000000, -Sqrt2, 0.000000},
		{-0.000000, 1.000000, -0.000000, 0.000000, -1.000000, Sqrt2, -Sqrt2, -1.000000, Sqrt2},
		{-0.000000, 0.000000, 1.000000, Sqrt2, -Sqrt2, 1.000000, -0.000000, -Sqrt2, 1.000000},
		{-1.000000, 0.000000, Sqrt2, 1.000000, -0.000000, 0.000000, -1.000000, -Sqrt2, Sqrt2},
		{0.000000, -1.000000, Sqrt2, -0.000000, 1.000000, -0.000000, -Sqrt2, 1.000000, 0.000000},
		{-Sqrt2, Sqrt2, 1.000000, 0.000000, -0.000000, 1.000000, -Sqrt2, 0.000000, 1.000000},
		{-1.000000, Sqrt2, -0.000000, 1.000000, -Sqrt2, Sqrt2, -1.000000, 0.000000, -0.000000},
		{-Sqrt2, 1.000000, Sqrt2, Sqrt2, -1.000000, 0.000000, -0.000000, -1.000000, -0.000000},
		{0.000000, Sqrt2, -1.000000, Sqrt2, 0.000000, -1.000000, 0.000000, -0.000000, -1.000000}
	},
	{
		{1.000000, -0.000000, 0.000000, 1.000000, 0.000000, -Sqrt2, -1.000000, Sqrt2, 0.000000},
		{-0.000000, 1.000000, -0.000000, 0.000000, 1.000000, -Sqrt2, Sqrt2, 1.000000, -Sqrt2},
		{-0.000000, 0.000000, 1.000000, -Sqrt2, Sqrt2, -1.000000, -0.000000, Sqrt2, -1.000000},
		{-1.000000, 0.000000, Sqrt2, -1.000000, -0.000000, 0.000000, 1.000000, Sqrt2, -Sqrt2},
		{0.000000, -1.000000, Sqrt2, -0.000000, -1.000000, -0.000000, Sqrt2, -1.000000, 0.000000},
		{-Sqrt2, Sqrt2, 1.000000, 0.000000, -0.000000, -1.000000, Sqrt2, 0.000000, -1.000000},
		{-1.000000, Sqrt2, -0.000000, -1.000000, Sqrt2, -Sqrt2, 1.000000, 0.000000, -0.000000},
		{-Sqrt2, 1.000000, Sqrt2, -Sqrt2, 1.000000, 0.000000, -0.000000, 1.000000, -0.000000},
		{0.000000, Sqrt2, -1.000000, -Sqrt2, 0.000000, 1.000000, 0.000000, -0.000000, 1.000000}
	},
	{
		{-1.000000, -0.000000, 0.000000, -1.000000, 0.000000, Sqrt2, -1.000000, Sqrt2, 0.000000},
		{-0.000000, -1.000000, -0.000000, 0.000000, -1.000000, Sqrt2, Sqrt2, 1.000000, -Sqrt2},
		{-0.000000, 0.000000, -1.000000, Sqrt2, -Sqrt2, 1.000000, -0.000000, Sqrt2, -1.000000},
		{1.000000, 0.000000, -Sqrt2, 1.000000, -0.000000, 0.000000, 1.000000, Sqrt2, -Sqrt2},
		{0.000000, 1.000000, -Sqrt2, -0.000000, 1.000000, -0.000000, Sqrt2, -1.000000, 0.000000},
		{Sqrt2, -Sqrt2, -1.000000, 0.000000, -0.000000, 1.000000, Sqrt2, 0.000000, -1.000000},
		{1.000000, -Sqrt2, -0.000000, 1.000000, -Sqrt2, Sqrt2, 1.000000, 0.000000, -0.000000},
		{Sqrt2, -1.000000, -Sqrt2, Sqrt2, -1.000000, 0.000000, -0.000000, 1.000000, -0.000000},
		{0.000000, -Sqrt2, 1.000000, Sqrt2, 0.000000, -1.000000, 0.000000, -0.000000, 1.000000}
	},
	{
		{-1.000000, 0.000000, -0.000000, 1.000000, -0.000000, -Sqrt2, -1.000000, Sqrt2, -0.000000},
		{0.000000, -1.000000, 0.000000, -0.000000, 1.000000, -Sqrt2, Sqrt2, 1.000000, -Sqrt2},
		{0.000000, -0.000000, -1.000000, -Sqrt2, Sqrt2, -1.000000, 0.000000, Sqrt2, -1.000000},
		{1.000000, -0.000000, -Sqrt2, -1.000000, 0.000000, -0.000000, 1.000000, Sqrt2, -Sqrt2},
		{-0.000000, 1.000000, -Sqrt2, 0.000000, -1.000000, 0.000000, Sqrt2, -1.000000, -0.000000},
		{Sqrt2, -Sqrt2, -1.000000, -0.000000, 0.000000, -1.000000, Sqrt2, -0.000000, -1.000000},
		{1.000000, -Sqrt2, 0.000000, -1.000000, Sqrt2, -Sqrt2, 1.000000, -0.000000, 0.000000},
		{Sqrt2, -1.000000, -Sqrt2, -Sqrt2, 1.000000, -0.000000, 0.000000, 1.000000, 0.000000},
		{-0.000000, -Sqrt2, 1.000000, -Sqrt2, -0.000000, 1.000000, -0.000000, 0.000000, 1.000000}
	},
	{
		{-1.000000, 0.000000, -0.000000, -1.000000, -0.000000, Sqrt2, 1.000000, -Sqrt2, -0.000000},
		{0.000000, -1.000000, 0.000000, -0.000000, -1.000000, Sqrt2, -Sqrt2, -1.000000, Sqrt2},
		{0.000000, -0.000000, -1.000000, Sqrt2, -Sqrt2, 1.000000, 0.000000, -Sqrt2, 1.000000},
		{1.000000, -0.000000, -Sqrt2, 1.000000, 0.000000, -0.000000, -1.000000, -Sqrt2, Sqrt2},
		{-0.000000, 1.000000, -Sqrt2, 0.000000, 1.000000, 0.000000, -Sqrt2, 1.000000, -0.000000},
		{Sqrt2, -Sqrt2, -1.000000, -0.000000, 0.000000, 1.000000, -Sqrt2, -0.000000, 1.000000},
		{1.000000, -Sqrt2, 0.000000, 1.000000, -Sqrt2, Sqrt2, -1.000000, -0.000000, 0.000000},
		{Sqrt2, -1.000000, -Sqrt2, Sqrt2, -1.000000, -0.000000, 0.000000, -1.000000, 0.000000},
		{-0.000000, -Sqrt2, 1.000000, Sqrt2, -0.000000, -1.000000, -0.000000, 0.000000, -1.000000}
	},
	{
		{1.000000, 0.000000, -0.000000, 1.000000, -0.000000, -Sqrt2, 1.000000, -Sqrt2, -0.000000},
		{0.000000, 1.000000, 0.000000, -0.000000, 1.000000, -Sqrt2, -Sqrt2, -1.000000, Sqrt2},
		{0.000000, -0.000000, 1.000000, -Sqrt2, Sqrt2, -1.000000, 0.000000, -Sqrt2, 1.000000},
		{-1.000000, -0.000000, Sqrt2, -1.000000, 0.000000, -0.000000, -1.000000, -Sqrt2, Sqrt2},
		{-0.000000, -1.000000, Sqrt2, 0.000000, -1.000000, 0.000000, -Sqrt2, 1.000000, -0.000000},
		{-Sqrt2, Sqrt2, 1.000000, -0.000000, 0.000000, -1.000000, -Sqrt2, -0.000000, 1.000000},
		{-1.000000, Sqrt2, 0.000000, -1.000000, Sqrt2, -Sqrt2, -1.000000, -0.000000, 0.000000},
		{-Sqrt2, 1.000000, Sqrt2, -Sqrt2, 1.000000, -0.000000, 0.000000, -1.000000, 0.000000},
		{-0.000000, Sqrt2, -1.000000, -Sqrt2, -0.000000, 1.000000, -0.000000, 0.000000, -1.000000}
	},
	{
		{-1.000000, -0.000000, 0.000000, 1.000000, 0.000000, -Sqrt2, 1.000000, -Sqrt2, 0.000000},
		{-0.000000, -1.000000, -0.000000, 0.000000, 1.000000, -Sqrt2, -Sqrt2, -1.000000, Sqrt2},
		{-0.000000, 0.000000, -1.000000, -Sqrt2, Sqrt2, -1.000000, -0.000000, -Sqrt2, 1.000000},
		{1.000000, 0.000000, -Sqrt2, -1.000000, -0.000000, 0.000000, -1.000000, -Sqrt2, Sqrt2},
		{0.000000, 1.000000, -Sqrt2, -0.000000, -1.000000, -0.000000, -Sqrt2, 1.000000, 0.000000},
		{Sqrt2, -Sqrt2, -1.000000, 0.000000, -0.000000, -1.000000, -Sqrt2, 0.000000, 1.000000},
		{1.000000, -Sqrt2, -0.000000, -1.000000, Sqrt2, -Sqrt2, -1.000000, 0.000000, -0.000000},
		{Sqrt2, -1.000000, -Sqrt2, -Sqrt2, 1.000000, 0.000000, -0.000000, -1.000000, -0.000000},
		{0.000000, -Sqrt2, 1.000000, -Sqrt2, 0.000000, 1.000000, 0.000000, -0.000000, -1.000000}
	}
};

// For each corner : order of the intersection point within the 3 chamfer face vertex indices
static const int32 ChamferOrder[8][3] =
{
	{1, 1, 1},
	{1, 1, 5},
	{1, 5, 1},
	{5, 1, 1},
	{5, 5, 1},
	{5, 1, 5},
	{1, 5, 5},
	{5, 5, 5}
};

// For each corner : order of the intersection point within the 2 chamfer + principal face vertex indices
// (9 combinations for the choice of the 2 chamfer + 1 principal)
static const int32 PrincipalOrder[8][9][3] =
{
	{
		{0, 3, 0},
		{1, 1, 1},
		{2, 2, 0},
		{0, 2, 2},
		{0, 0, 3},
		{1, 1, 1},
		{1, 1, 1},
		{2, 0, 2},
		{3, 0, 0}
	},
	{
		{2, 0, 7},
		{2, 2, 6},
		{1, 1, 3},
		{7, 1, 5},
		{3, 6, 4},
		{2, 2, 6},
		{0, 2, 6},
		{1, 1, 5},
		{0, 3, 0}
	},
	{
		{6, 4, 3},
		{2, 6, 2},
		{1, 5, 7},
		{1, 5, 1},
		{3, 0, 0},
		{2, 6, 0},
		{2, 6, 2},
		{1, 3, 1},
		{0, 7, 2}
	},
	{
		{0, 0, 3},
		{6, 0, 2},
		{5, 1, 1},
		{3, 1, 1},
		{7, 2, 0},
		{6, 2, 2},
		{6, 2, 2},
		{5, 7, 1},
		{4, 3, 6}
	},
	{
		{2, 7, 0},
		{5, 3, 1},
		{6, 6, 2},
		{6, 6, 2},
		{4, 6, 3},
		{5, 5, 7},
		{5, 5, 1},
		{6, 4, 2},
		{7, 4, 4}
	},
	{
		{6, 3, 4},
		{5, 7, 5},
		{6, 2, 6},
		{4, 2, 6},
		{4, 4, 7},
		{5, 1, 5},
		{3, 1, 5},
		{6, 2, 6},
		{7, 0, 2}
	},
	{
		{4, 7, 4},
		{1, 5, 5},
		{2, 6, 4},
		{2, 6, 6},
		{0, 2, 7},
		{1, 5, 3},
		{7, 5, 5},
		{2, 6, 6},
		{3, 4, 6}
	},
	{
		{4, 4, 7},
		{6, 4, 6},
		{5, 5, 5},
		{5, 5, 5},
		{7, 4, 4},
		{6, 6, 4},
		{4, 6, 6},
		{5, 5, 5},
		{4, 7, 4}
	}
};

void FTribox::BuildPrincipalDist(const FVec3Type& P, const int32 CoordIndexA, const int32 DistsIndexA, const int32 DistsIndexB)
{
	MaxDists[DistsIndexA] = FMath::Max(P[CoordIndexA], MaxDists[DistsIndexA]);
	MaxDists[DistsIndexB] = FMath::Max(-P[CoordIndexA], MaxDists[DistsIndexB]);
}

void FTribox::BuildChamferDist(const FVec3Type& P, const int32 CoordIndexA, const int32 CoordIndexB,
		const int32 DistsIndexA, const int32 DistsIndexB, const int32 DistsIndexC, const int32 DistsIndexD)
{
	MaxDists[DistsIndexA] = FMath::Max(P[CoordIndexA]+P[CoordIndexB], MaxDists[DistsIndexA]);
	MaxDists[DistsIndexB] = FMath::Max(-P[CoordIndexA]-P[CoordIndexB], MaxDists[DistsIndexB]);
	MaxDists[DistsIndexC] = FMath::Max(P[CoordIndexA]-P[CoordIndexB], MaxDists[DistsIndexC]);
	MaxDists[DistsIndexD] = FMath::Max(-P[CoordIndexA]+P[CoordIndexB], MaxDists[DistsIndexD]);
}

void FTribox::AddPoint(const FVec3Type& PointPosition)
{
	const FVec3 P = PointPosition;

	// Build the max distance along each principal axis
	BuildPrincipalDist(P, 0,0,1);
	BuildPrincipalDist(P, 1,2,3);
	BuildPrincipalDist(P, 2,4,5);

	// Build the max distance along each chamfer axis
	BuildChamferDist(P, 1, 2, 6, 7, 8, 9);
	BuildChamferDist(P, 0, 2, 10, 11, 12, 13);
	BuildChamferDist(P, 0, 1, 14, 15, 16, 17);

	bHasDatas = true;
}

DECLARE_CYCLE_STAT(TEXT("Collisions::AddConvex"), STAT_AddConvexToTribox, STATGROUP_ChaosCollision);
void FTribox::AddConvex(const FConvex* Convex, const FRigidTransform3Type& RelativeTransform)
{
	SCOPE_CYCLE_COUNTER(STAT_AddConvexToTribox);
	if(Convex)
	{
		for(int32 VertexIndex = 0; VertexIndex < Convex->NumVertices(); ++VertexIndex)
		{
			AddPoint(RelativeTransform.TransformPosition(FVec3Type(
				Convex->GetVertex(VertexIndex)[0],
				Convex->GetVertex(VertexIndex)[1],
				Convex->GetVertex(VertexIndex)[2])));
		}
	}
}

bool FTribox::BuildTribox()
{
	if(bHasDatas)
	{
		// Inflate the principal planes max distance by a given distance
		for(int32 PlaneIndex = 0; PlaneIndex < NumPrincipalPlanes; ++PlaneIndex)
		{
			MaxDists[PlaneIndex] += InflateDistance;
		}
		// Scale the chamfer planes max distance  by 1 / sqrt(2) + Inflate by a given distance
		for(int32 PlaneIndex = NumPrincipalPlanes; PlaneIndex < NumPlanes; ++PlaneIndex)
		{
			MaxDists[PlaneIndex] = MaxDists[PlaneIndex] * InvSqrt2 + InflateDistance;
		}
		return true;
	}
	return false;
}

FVec3 FTribox::SolveIntersection(const int32 FaceIndex[3], const FMatrix33Type& A) const
{
	const FVec3 B(MaxDists[FaceIndex[0]],MaxDists[FaceIndex[1]],MaxDists[FaceIndex[2]]);
	
	return  FVec3(
	A.GetAt(0,0) * B[0] + A.GetAt(0,1) * B[1] + A.GetAt(0,2) * B[2],
	A.GetAt(1,0) * B[0] + A.GetAt(1,1) * B[1] + A.GetAt(1,2) * B[2],
	A.GetAt(2,0) * B[0] + A.GetAt(2,1) * B[1] + A.GetAt(2,2) * B[2]);
}

void FTribox::AddIntersection(const int32 FaceIndex[3], const int32 FaceOrder[3], const FVec3Type& VertexPosition,
		TArray<TArray<int32>>& FaceIndices, TArray<FConvex::FVec3Type>& ConvexVertices) const
{
	const int32 VertexOffset = ConvexVertices.Num(); 

	FaceIndices[FaceIndex[0]][FaceOrder[0]] = VertexOffset;
	FaceIndices[FaceIndex[1]][FaceOrder[1]] = VertexOffset;
	FaceIndices[FaceIndex[2]][FaceOrder[2]] = VertexOffset;

	ConvexVertices.Add(VertexPosition);
}

void FTribox::ComputeIntersection(const int32 CornerIndex, const int32 PrincipalIndex, const int32 ChamferIndex,
	TArray<TArray<int32>>& FaceIndices, TArray<FConvex::FVec3Type>& ConvexVertices) const
{
	// Compute the intersection in between 2 corner chamfer planes and one principal one
	int32 TId[3] = {ChamferIndices[CornerIndex][0], ChamferIndices[CornerIndex][1], ChamferIndices[CornerIndex][2]};
	TId[ChamferIndex] = PrincipalIndices[CornerIndex][PrincipalIndex];

	const int32 LinearIndex = PrincipalIndex*3+ChamferIndex;

	const FVec3Type V = SolveIntersection(TId, PrincipalMatrices[CornerIndex][LinearIndex]);
	AddIntersection(TId, PrincipalOrder[CornerIndex][LinearIndex], V, FaceIndices, ConvexVertices);
}

FORCEINLINE void CompressFaces(TArray<TArray<int32>>& FaceIndices)
{
	TArray<int32> CompressedIndices;
	CompressedIndices.Reserve(8);
	for(int32 PlaneIndex = 0; PlaneIndex < FaceIndices.Num(); ++PlaneIndex)
	{
		CompressedIndices.Reset();
		for(int32 LocalIndex = FaceIndices[PlaneIndex].Num()-1; LocalIndex >= 0; --LocalIndex)
		{
			int32 VertexIndex = FaceIndices[PlaneIndex][LocalIndex];
			if(VertexIndex != INDEX_NONE)
			{
				CompressedIndices.Add(VertexIndex);
			}
		}
		FaceIndices[PlaneIndex] = CompressedIndices;
	}
}

FImplicitObjectPtr FTribox::CreateConvexFromTopology(
	TArray<FConvex::FPlaneType>&& ConvexPlanes, TArray<TArray<int32>>&& FaceIndices, TArray<FConvex::FVec3Type>&& ConvexVertices) const
{
	const FRealType HX = (MaxDists[0]+MaxDists[1]);
	const FRealType HY = (MaxDists[2]+MaxDists[3]);
	const FRealType HZ = (MaxDists[4]+MaxDists[5]);
	
	const FVec3Type InertiaTensor = FVec3Type(HY * HY + HZ * HZ, HX * HX + HZ * HZ, HY * HY + HX * HX) / (12.0f);
	const FRealType Volume = HX * HY * HZ;
	
	return MakeImplicitObjectPtr<Chaos::FConvex>(MoveTemp(ConvexPlanes), MoveTemp(FaceIndices), MoveTemp(ConvexVertices), 
		FConvex::FVec3Type(-MaxDists[1], -MaxDists[3], -MaxDists[5]), FConvex::FVec3Type(MaxDists[0], MaxDists[2], MaxDists[4]), 
		Volume, InertiaTensor, FRotation3::Identity, true);
}

DECLARE_CYCLE_STAT(TEXT("Collisions::MakeConvex"), STAT_MakeTriboxConvex, STATGROUP_ChaosCollision);
FImplicitObjectPtr FTribox::MakeConvex() const
{
	TArray<TArray<int32>> FaceIndices;
	TArray<FConvex::FVec3Type> ConvexVertices;
	TArray<FConvex::FPlaneType> ConvexPlanes;
	{
		SCOPE_CYCLE_COUNTER(STAT_MakeTriboxConvex);

		FaceIndices.SetNum(NumPlanes);
		ConvexVertices.Reserve(32);
		ConvexPlanes.SetNum(NumPlanes);

		for(int32 PlaneIndex = 0; PlaneIndex < NumPlanes; ++PlaneIndex)
		{
			ConvexPlanes[PlaneIndex] = FConvex::FPlaneType(
				PlanesDirs[PlaneIndex] * MaxDists[PlaneIndex], PlanesDirs[PlaneIndex]);
			FaceIndices[PlaneIndex].Init(INDEX_NONE, 8);
		}
		for(int32 CornerIndex = 0; CornerIndex < 8; ++CornerIndex)
		{
			// Compute the intersection in between 3 corner chamfer planes
			const FVec3Type P = SolveIntersection(ChamferIndices[CornerIndex], ChamferMatrices[CornerIndex]);

			if(P.Dot(PlanesDirs[PrincipalIndices[CornerIndex][0]]) > MaxDists[PrincipalIndices[CornerIndex][0]])
			{
				ComputeIntersection(CornerIndex, 0,1, FaceIndices, ConvexVertices);
				ComputeIntersection(CornerIndex, 0,2, FaceIndices, ConvexVertices);
				ComputeIntersection(CornerIndex, 1,1, FaceIndices, ConvexVertices);
				ComputeIntersection(CornerIndex, 2,2, FaceIndices, ConvexVertices);
			}
			else if(P.Dot(PlanesDirs[PrincipalIndices[CornerIndex][1]]) > MaxDists[PrincipalIndices[CornerIndex][1]])
			{
				ComputeIntersection(CornerIndex, 1,0, FaceIndices, ConvexVertices);
				ComputeIntersection(CornerIndex, 1,2, FaceIndices, ConvexVertices);
				ComputeIntersection(CornerIndex, 0,0, FaceIndices, ConvexVertices);
				ComputeIntersection(CornerIndex, 2,2, FaceIndices, ConvexVertices);
			}
			else if(P.Dot(PlanesDirs[PrincipalIndices[CornerIndex][2]]) > MaxDists[PrincipalIndices[CornerIndex][2]])
			{
				ComputeIntersection(CornerIndex, 2,0, FaceIndices, ConvexVertices);
				ComputeIntersection(CornerIndex, 2,1, FaceIndices, ConvexVertices);
				ComputeIntersection(CornerIndex, 0,0, FaceIndices, ConvexVertices);
				ComputeIntersection(CornerIndex, 1,1, FaceIndices, ConvexVertices);
			}
			else
			{
				ComputeIntersection(CornerIndex, 0,0, FaceIndices, ConvexVertices);
				ComputeIntersection(CornerIndex, 1,1, FaceIndices, ConvexVertices);
				ComputeIntersection(CornerIndex, 2,2, FaceIndices, ConvexVertices);

				AddIntersection(ChamferIndices[CornerIndex], ChamferOrder[CornerIndex], P, FaceIndices, ConvexVertices);
			}
		}
		CompressFaces(FaceIndices);
	}

	return CreateConvexFromTopology(MoveTemp(ConvexPlanes), MoveTemp(FaceIndices), MoveTemp(ConvexVertices));
}

bool FTribox::IsTriboxOverlapping(const FTribox& OtherTribox) const
{
	for(int32 AxisIndex = 0; AxisIndex < NumPlanes; AxisIndex += 2)
	{
		if((MaxDists[AxisIndex] < -OtherTribox.MaxDists[AxisIndex+1]) ||
			(OtherTribox.MaxDists[AxisIndex] < -MaxDists[AxisIndex+1]))
		{
			return false;
		}
	}
	return true;
}

FAABB3 FTribox::GetBounds() const
{
	return FAABB3( FVector(-MaxDists[1], -MaxDists[3], -MaxDists[5]), FVector(MaxDists[0], MaxDists[2], MaxDists[4]));
}
	
bool FTribox::OverlapTribox(const FTribox& OtherTribox, FTribox& OverlapTribox) const
{
	bool bOverlapTriboxes = IsTriboxOverlapping(OtherTribox);
	if(bOverlapTriboxes)
	{
		for(int32 AxisIndex = 0; AxisIndex < NumPlanes; AxisIndex += 2)
		{
			OverlapTribox.MaxDists[AxisIndex] = FMath::Min(MaxDists[AxisIndex], OtherTribox.MaxDists[AxisIndex]);
			OverlapTribox.MaxDists[AxisIndex+1] = FMath::Min(MaxDists[AxisIndex+1], OtherTribox.MaxDists[AxisIndex+1]);
		}
	}
	return bOverlapTriboxes;
}

bool FTribox::SplitTriboxSlab(const int32 PlaneAxis, const FRealType& PlaneDistance,
	FTribox& LeftTribox, FTribox& RightTribox) const
{
	if(PlaneDistance < MaxDists[PlaneAxis])
	{
		if(PlaneDistance > -MaxDists[PlaneAxis+1])
		{
			LeftTribox = *this;
			RightTribox = *this;

			auto CutTribox = [](const int32 PX, const int32 MX, const int32 PY, const int32 MY,
						const int32 PZPX, const int32 PZMX, const int32 PZPY, const int32 PZMY,
						const int32 MZPX, const int32 MZMX, const int32 MZPY, const int32 MZMY,
						const int32 PXPY, const int32 PXMY, const int32 MXPY, const int32 MXMY,
						const int32 PZ, const FRealType MaxDistance, FTribox& SideTribox)
			{
				SideTribox.MaxDists[PZ] = MaxDistance;
				
				SideTribox.MaxDists[PX] = FMath::Min(SideTribox.MaxDists[PX], SideTribox.MaxDists[MZPX] * Sqrt2 + MaxDistance);
				SideTribox.MaxDists[MX] = FMath::Min(SideTribox.MaxDists[MX], SideTribox.MaxDists[MZMX] * Sqrt2 + MaxDistance);
				SideTribox.MaxDists[PY] = FMath::Min(SideTribox.MaxDists[PY], SideTribox.MaxDists[MZPY] * Sqrt2 + MaxDistance);
				SideTribox.MaxDists[MY] = FMath::Min(SideTribox.MaxDists[MY], SideTribox.MaxDists[MZMY] * Sqrt2 + MaxDistance);

				SideTribox.MaxDists[PZPX] = FMath::Min(SideTribox.MaxDists[PZPX], (SideTribox.MaxDists[PX] + MaxDistance) * InvSqrt2);
				SideTribox.MaxDists[PZMX] = FMath::Min(SideTribox.MaxDists[PZMX], (SideTribox.MaxDists[MX] + MaxDistance) * InvSqrt2);
				SideTribox.MaxDists[PZPY] = FMath::Min(SideTribox.MaxDists[PZPY], (SideTribox.MaxDists[PY] + MaxDistance) * InvSqrt2);
				SideTribox.MaxDists[PZMY] = FMath::Min(SideTribox.MaxDists[PZMY], (SideTribox.MaxDists[MY] + MaxDistance) * InvSqrt2);
				
				SideTribox.MaxDists[PXPY] = FMath::Min(SideTribox.MaxDists[PXPY], (SideTribox.MaxDists[PX] + SideTribox.MaxDists[PY]) * InvSqrt2);
				SideTribox.MaxDists[PXMY] = FMath::Min(SideTribox.MaxDists[PXMY], (SideTribox.MaxDists[PX] + SideTribox.MaxDists[MY]) * InvSqrt2);
				SideTribox.MaxDists[MXPY] = FMath::Min(SideTribox.MaxDists[MXPY], (SideTribox.MaxDists[MX] + SideTribox.MaxDists[PY]) * InvSqrt2);
				SideTribox.MaxDists[MXMY] = FMath::Min(SideTribox.MaxDists[MXMY], (SideTribox.MaxDists[MX] + SideTribox.MaxDists[MY]) * InvSqrt2);
			};
			
			if(PlaneAxis == 0)
			{
				CutTribox(4,5,2,3,10,12,14,16,13,11,17,15,6,9,8,7,0, PlaneDistance, LeftTribox);
				CutTribox(4,5,2,3,13,11,17,15,10,12,14,16,6,9,8,7,1,-PlaneDistance, RightTribox);
			}
			else if(PlaneAxis == 2)
			{
				CutTribox(4,5,0,1,6,8,14,17,9,7,16,15,10,13,12,11,2, PlaneDistance, LeftTribox);
				CutTribox(4,5,0,1,9,7,16,15,6,8,14,17,10,13,12,11,3,-PlaneDistance, RightTribox);
			}
			else if(PlaneAxis == 4)
			{
				CutTribox(2,3,0,1,6,9,10,13,8,7,12,11,14,17,16,15,4, PlaneDistance, LeftTribox);
				CutTribox(2,3,0,1,8,7,12,11,6,9,10,13,14,17,16,15,5,-PlaneDistance, RightTribox);
			}
			
			return true;
		}
		else  
		{
			RightTribox = *this;
		}
	}
	else
	{
		LeftTribox = *this;
	}
	return false;
}

int32 FTribox::GetThickestSlab() const
{
	int32 ThickestSlab = INDEX_NONE;
	FRealType MaxThickness = 0.0;
	for(int32 AxisIndex = 0; AxisIndex < NumPrincipalPlanes; AxisIndex += 2)
	{
		const FRealType SlabThickness = (MaxDists[AxisIndex]+MaxDists[AxisIndex+1]);
		if(SlabThickness > MaxThickness)
		{
			ThickestSlab = AxisIndex;
			MaxThickness = SlabThickness;
		}
	}
	return ThickestSlab;
}

FTribox::FRealType FTribox::SampleSlabPoint(const int32 PlaneAxis, const FRealType& LocalDistance) const
{
	return -MaxDists[PlaneAxis+1] + FMath::Max(0.0f, FMath::Min(1.0f, LocalDistance)) * (MaxDists[PlaneAxis]+MaxDists[PlaneAxis+1]);
}
	
FTribox::FRealType FTribox::GetClosestPlane(const FVec3Type& PointPosition, int32& PlaneAxis, FRealType& PlaneProjection) const
{
	FRealType ClosestDistance = -FLT_MAX;
	for(int32 AxisIndex = 0; AxisIndex < 3; ++AxisIndex)
	{
		const int32 PlaneIndex = 2 * AxisIndex;
		FRealType PlaneDistance = PointPosition[AxisIndex]-MaxDists[PlaneIndex];
		if(PlaneDistance > ClosestDistance)
		{
			ClosestDistance = PlaneDistance;
			PlaneAxis = PlaneIndex;
			PlaneProjection = MaxDists[PlaneIndex];
		}
		PlaneDistance = -PointPosition[AxisIndex]-MaxDists[PlaneIndex+1];
		if(PlaneDistance > ClosestDistance)
		{
			ClosestDistance = PlaneDistance;
			PlaneAxis = PlaneIndex;
			PlaneProjection = -MaxDists[PlaneIndex+1];
		}
	}
	return ClosestDistance;
}

FTribox::FRealType FTribox::ComputeVolume() const
{
	const FVec3Type BoxSize(MaxDists[0]+MaxDists[1], MaxDists[2]+MaxDists[3], MaxDists[4]+MaxDists[5]);
	FRealType Volume = BoxSize[0] * BoxSize[1] * BoxSize[2];

	return Volume;
}
	
FTribox::FVec3Type FTribox::GetCenter() const
{
	return 0.5f * FVec3Type(MaxDists[0]-MaxDists[1], MaxDists[2]-MaxDists[3], MaxDists[4]-MaxDists[5]);
}
	
FTribox& FTribox::operator+=(const FTribox& OtherTribox)
{
	for(int32 PlaneIndex = 0; PlaneIndex < NumPlanes; ++PlaneIndex)
	{
		MaxDists[PlaneIndex] = FMath::Max(OtherTribox.MaxDists[PlaneIndex], MaxDists[PlaneIndex]);
	}
	bHasDatas |= OtherTribox.bHasDatas;
	return *this;
}

FTribox FTribox::operator+(const FTribox& OtherTribox) const
{
	FTribox ReturnTribox(*this);
	return ReturnTribox += OtherTribox;
}
	
}
}
	