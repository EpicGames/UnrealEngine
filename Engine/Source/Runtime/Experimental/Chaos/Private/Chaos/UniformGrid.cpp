// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/UniformGrid.h"

#include "Chaos/ArrayFaceND.h"
#include "Chaos/ArrayND.h"
#include "Chaos/Core.h"

namespace Chaos
{
	template<class T_SCALAR, class T>
	T_SCALAR LinearlyInterpolate1D(const T_SCALAR& Prev, const T_SCALAR& Next, const T Alpha)
	{
		return  Prev + (Next - Prev) * Alpha;
	}

	template<class T_SCALAR, class T, int d>
	T_SCALAR LinearlyInterpolateHelper(const TArrayND<T_SCALAR, d>& ScalarN, const TVector<int32, d>& CellPrev, const TVector<T, d>& Alpha)
	{
		check(false);
	}

	template<class T_SCALAR, class T>
	T_SCALAR LinearlyInterpolateHelper(const TArrayND<T_SCALAR, 2>& ScalarN, const TVector<int32, 2>& CellPrev, const TVector<T, 2>& Alpha)
	{
		const T_SCALAR interpx1 = LinearlyInterpolate1D(ScalarN(CellPrev), ScalarN(CellPrev + TVector<int32, 2>({ 1, 0 })), Alpha[0]);
		const T_SCALAR interpx2 = LinearlyInterpolate1D(ScalarN(CellPrev + TVector<int32, 2>({ 0, 1 })), ScalarN(CellPrev + TVector<int32, 2>({ 1, 1 })), Alpha[0]);
		return LinearlyInterpolate1D(interpx1, interpx2, Alpha[1]);
	}

	template<class T_SCALAR, class T>
	T_SCALAR LinearlyInterpolateHelper(const TArrayND<T_SCALAR, 3>& ScalarN, const TVector<int32, 3>& CellPrev, const TVector<T, 3>& Alpha)
	{
		const T_SCALAR interpx1 = LinearlyInterpolate1D(ScalarN(CellPrev), ScalarN(CellPrev + TVector<int32, 3>(1, 0, 0)), Alpha[0]);
		const T_SCALAR interpx2 = LinearlyInterpolate1D(ScalarN(CellPrev + TVector<int32, 3>(0, 1, 0)), ScalarN(CellPrev + TVector<int32, 3>(1, 1, 0)), Alpha[0]);
		const T_SCALAR interpx3 = LinearlyInterpolate1D(ScalarN(CellPrev + TVector<int32, 3>(0, 0, 1)), ScalarN(CellPrev + TVector<int32, 3>(1, 0, 1)), Alpha[0]);
		const T_SCALAR interpx4 = LinearlyInterpolate1D(ScalarN(CellPrev + TVector<int32, 3>(0, 1, 1)), ScalarN(CellPrev + TVector<int32, 3>(1, 1, 1)), Alpha[0]);
		const T_SCALAR interpy1 = LinearlyInterpolate1D(interpx1, interpx2, Alpha[1]);
		const T_SCALAR interpy2 = LinearlyInterpolate1D(interpx3, interpx4, Alpha[1]);
		return LinearlyInterpolate1D(interpy1, interpy2, Alpha[2]);
	}

	template<class T, int d>
	template<class T_SCALAR>
	T_SCALAR TUniformGridBase<T, d>::LinearlyInterpolate(const TArrayND<T_SCALAR, d>& ScalarN, const TVector<T, d>& X) const
	{
		TVector<int32, d> XCell = Cell(X);
		TVector<T, d> XCenter = Location(XCell);
		TVector<int32, d> CellPrev;
		for(int32 i = 0; i < d; ++i)
		{
			CellPrev[i] = X[i] > XCenter[i] ? XCell[i] : XCell[i] - 1;
		}
		TVector<T, d> Alpha = (X - Location(CellPrev)) / MDx;
		// Clamp correctly when on boarder
		for(int32 i = 0; i < d; ++i)
		{
			if(CellPrev[i] == -1)
			{
				CellPrev[i] = 0;
				Alpha[i] = 0;
			}
			if(CellPrev[i] == Counts()[i] - 1)
			{
				CellPrev[i] = Counts()[i] - 2;
				Alpha[i] = 1;
			}
		}
		return LinearlyInterpolateHelper(ScalarN, CellPrev, Alpha);
	}

	template<class T, int d>
	T TUniformGridBase<T, d>::LinearlyInterpolateComponent(const TArrayND<T, d>& ScalarNComponent, const TVector<T, d>& X, const int32 Axis) const
	{
		TVector<int32, d> CellCounts = Counts() + TVector<int32, d>::AxisVector(Axis);
		TVector<int32, d> FaceIndex = Face(X, Axis);
		TVector<T, d> XCenter = Location(MakePair(Axis, FaceIndex));
		TVector<int32, d> FacePrev;
		for(int32 i = 0; i < d; ++i)
		{
			FacePrev[i] = X[i] > XCenter[i] ? FaceIndex[i] : FaceIndex[i] - 1;
		}
		TVector<T, d> Alpha = (X - Location(MakePair(Axis, FacePrev))) / MDx;
		// Clamp correctly when on boarder
		for(int32 i = 0; i < d; ++i)
		{
			if(FacePrev[i] == -1)
			{
				FacePrev[i] = 0;
				Alpha[i] = 0;
			}
			if(FacePrev[i] == CellCounts[i] - 1)
			{
				FacePrev[i] = CellCounts[i] - 2;
				Alpha[i] = 1;
			}
		}
		return LinearlyInterpolateHelper(ScalarNComponent, FacePrev, Alpha);
	}

	template<class T, int d>
	TVector<T, d> TUniformGridBase<T, d>::LinearlyInterpolate(const TArrayFaceND<T, d>& ScalarN, const TVector<T, d>& X) const
	{
		TVector<T, d> Result;
		for(int32 i = 0; i < d; ++i)
		{
			Result[i] = LinearlyInterpolateComponent(ScalarN.GetComponent(i), X, i);
		}
		return Result;
	}

	template<class T, int d>
	TVector<T, d> TUniformGridBase<T, d>::LinearlyInterpolate(const TArrayFaceND<T, d>& ScalarN, const TVector<T, d>& X, const Pair<int32, TVector<int32, d>> Index) const
	{
		TVector<T, d> Result;
		for(int32 i = 0; i < d; ++i)
		{
			if(i == Index.First)
			{
				Result[i] = ScalarN(Index);
			}
			else
			{
				Result[i] = LinearlyInterpolateComponent(ScalarN.GetComponent(i), X, Index.First);
			}
		}
		return Result;
	}

	template<class T, int d>
	TVector<int32, d> TUniformGrid<T, d>::GetIndex(const int32 Index) const
	{
		TVector<int32, d> NDIndex;
		int32 product = 1, Remainder = Index;
		for(int32 i = 0; i < d; ++i)
		{
			product *= MCells[i];
		}
		for(int32 i = 0; i < d; ++i)
		{
			product /= MCells[i];
			NDIndex[i] = Remainder / product;
			Remainder -= NDIndex[i] * product;
		}
		return NDIndex;
	}

	template<class T, int d>
	TVector<T, d> TUniformGrid<T, d>::Clamp(const TVector<T, d>& X) const
	{
		TVector<T, d> Result;
		for(int32 i = 0; i < d; ++i)
		{
			if(X[i] > MMaxCorner[i])
				Result[i] = MMaxCorner[i];
			else if(X[i] < MMinCorner[i])
				Result[i] = MMinCorner[i];
			else
				Result[i] = X[i];
		}
		return Result;
	}

	template<class T, int d>
	TVector<T, d> TUniformGrid<T, d>::ClampMinusHalf(const TVector<T, d>& X) const
	{
		TVector<T, d> Result;
		TVector<T, d> Max = MMaxCorner - MDx * 0.5;
		TVector<T, d> Min = MMinCorner + MDx * 0.5;
		for(int32 i = 0; i < d; ++i)
		{
			if(X[i] > Max[i])
				Result[i] = Max[i];
			else if(X[i] < Min[i])
				Result[i] = Min[i];
			else
				Result[i] = X[i];
		}
		return Result;
	}

	template<class T>
	TVector<int32, 3> TUniformGrid<T, 3>::GetIndex(const int32 Index) const
	{
		int32 Remainder;
		TVector<int32, 3> NDIndex;
		NDIndex[0] = Index / (MCells[1] * MCells[2]);
		Remainder = Index - NDIndex[0] * MCells[1] * MCells[2];
		NDIndex[1] = Remainder / MCells[2];
		Remainder = Remainder - NDIndex[1] * MCells[2];
		NDIndex[2] = Remainder;
		return NDIndex;
	}

	template<class T>
	Pair<int32, TVector<int32, 3>> TUniformGrid<T, 3>::GetFaceIndex(int32 Index) const
	{
		int32 Remainder;
		int32 NumXFaces = (MCells + TVector<int32, 3>::AxisVector(0)).Product();
		int32 NumYFaces = (MCells + TVector<int32, 3>::AxisVector(1)).Product();
		int32 Axis = 0;
		if(Index > NumXFaces)
		{
			Axis = 1;
			Index -= NumXFaces;
			if(Index > NumYFaces)
			{
				Axis = 2;
				Index -= NumYFaces;
			}
		}
		TVector<int32, 3> Faces = MCells + TVector<int32, 3>::AxisVector(Axis);
		TVector<int32, 3> NDIndex;
		NDIndex[0] = Index / (Faces[1] * Faces[2]);
		Remainder = Index - NDIndex[0] * Faces[1] * Faces[2];
		NDIndex[1] = Remainder / Faces[2];
		Remainder = Remainder - NDIndex[1] * Faces[2];
		NDIndex[2] = Remainder;
		return MakePair(Axis, NDIndex);
	}

	template<class T>
	TVector<int32, 3> TUniformGrid<T, 3>::ClampIndex(const TVector<int32, 3>& Index) const
	{
		TVector<int32, 3> Result;
		Result[0] = Index[0] >= MCells[0] ? MCells[0] - 1 : (Index[0] < 0 ? 0 : Index[0]);
		Result[1] = Index[1] >= MCells[1] ? MCells[1] - 1 : (Index[1] < 0 ? 0 : Index[1]);
		Result[2] = Index[2] >= MCells[2] ? MCells[2] - 1 : (Index[2] < 0 ? 0 : Index[2]);
		return Result;
	}

	template<class T>
	TVector<T, 3> TUniformGrid<T, 3>::Clamp(const TVector<T, 3>& X) const
	{
		TVector<T, 3> Result;
		Result[0] = X[0] > MMaxCorner[0] ? MMaxCorner[0] : (X[0] < MMinCorner[0] ? MMinCorner[0] : X[0]);
		Result[1] = X[1] > MMaxCorner[1] ? MMaxCorner[1] : (X[1] < MMinCorner[1] ? MMinCorner[1] : X[1]);
		Result[2] = X[2] > MMaxCorner[2] ? MMaxCorner[2] : (X[2] < MMinCorner[2] ? MMinCorner[2] : X[2]);
		return Result;
	}

	template<class T>
	TVector<T, 3> TUniformGrid<T, 3>::ClampMinusHalf(const TVector<T, 3>& X) const
	{
		TVector<T, 3> Result;
		TVector<T, 3> Max = MMaxCorner - MDx * 0.5;
		TVector<T, 3> Min = MMinCorner + MDx * 0.5;
		Result[0] = X[0] > Max[0] ? Max[0] : (X[0] < Min[0] ? Min[0] : X[0]);
		Result[1] = X[1] > Max[1] ? Max[1] : (X[1] < Min[1] ? Min[1] : X[1]);
		Result[2] = X[2] > Max[2] ? Max[2] : (X[2] < Min[2] ? Min[2] : X[2]);
		return Result;
	}

	template<class T>
	TUniformGrid<T, 3> TUniformGrid<T, 3>::SubGrid(const TVector<int32, 3>& MinCell, const TVector<int32, 3>& MaxCell) const
	{
		TUniformGrid<T, 3> Result;
		Result.MMinCorner = Node(MinCell);
		Result.MMaxCorner = Node(MaxCell + TVector<int32, 3>(1, 1, 1));
		Result.MCells = MaxCell - MinCell + TVector<int32, 3>(1, 1, 1);
		Result.MDx = MDx;
		return Result;
	}

	template<class T>
	void TMPMGrid<T>::BaseNodeIndex(const TVector<T, 3>& X, TVector<int32, 3>& Index, TVector<T, 3>& Weights) const
	{
		for(uint32 i = 0; i < 3; ++i)
			Index[i] = int32(FMath::Floor((X[i] - MMinCorner[i]) / MDx[i]));
		for(uint32 i = 0; i < 3; ++i)
			Weights[i] = (X[i] - MMinCorner[i]) / MDx[i] - T(Index[i]);
	}

	template<class T>
	int32 TMPMGrid<T>::FlatIndex(const TVector<int32, 3>& Index) const
	{
		/*
		This assumes data will be stored at each index (i.e. numbered by flatIndex(index)). For linear interpolation over
		the grid we assume that the index[i] \in [0,gridN[i]-1]. For quadratic interpolation, we assume that the
		index[i] \in [-1,gridN[i]-1].

		For linear interpolation, data is stored at the nodes of the grid, e.g. xi = i dx (i=0,1,...,gridN-1) and
		for quadratic interpolation, data is stored at the cell centers of the grid, e.g. xi = i dx + dx/2 (i=-1,0,1,...,gridN-1)
		*/

		if(interp == linear)
		{
			return Index[0] * MCells[1] * MCells[2] + Index[1] * MCells[2] + Index[2];
		}
		else
		{
			return (MCells[1] + 1) * (MCells[2] + 1) * (Index[0] + 1) + (MCells[2] + 1) * (Index[1] + 1) + Index[2] + 1;
		}
	}

	template<class T>
	TVector<int32, 3> TMPMGrid<T>::Lin2MultiIndex(const int32 IndexIn) const
	{
		/*
		This version returns an Index instead of taking one in by reference.
		*/
		TVector<int32, 3> MultiIndex;

		if(interp == linear)
		{
			MultiIndex[0] = IndexIn / (MCells[1] * MCells[2]);
			MultiIndex[1] = (IndexIn / MCells[2]) % MCells[1];
			MultiIndex[2] = IndexIn % MCells[2];
		}
		else
		{
			MultiIndex[0] = IndexIn / ((MCells[1] + 1) * (MCells[2] + 1)) - 1;
			MultiIndex[1] = (IndexIn / (MCells[2] + 1)) % (MCells[1] + 1) - 1;
			MultiIndex[2] = IndexIn % (MCells[2] + 1) - 1;
		}


		return MultiIndex;
	}



	template<class T>
	TVector<T, 3> TMPMGrid<T>::Node(int32 FlatIndexIn) const
	{
		TVector<T, 3> result;
		TVector<int32, 3> index = Lin2MultiIndex(FlatIndexIn);

		if(interp == linear)
		{
			for(int32 i = 0; i < 3; ++i)
			{
				result[i] = T(index[i]) * MDx[i] + MMinCorner[i];
			}
		}
		else
		{
			for(size_t i = 0; i < 3; ++i)
			{
				result[i] = (T(index[i]) + T(0.5)) * MDx[i] + MMinCorner[i];
			}
		}

		return result;
	}

	template<class T>
	int32 TMPMGrid<T>::Loc2GlobIndex(const int32 IndexIn, const TVector<int32, 3>& LocalIndexIn) const
	{

		if(interp == linear)
		{
			return IndexIn + MCells[1] * MCells[2] * LocalIndexIn[0] + MCells[2] * LocalIndexIn[1] + LocalIndexIn[2];
		}
		else
		{
			return IndexIn + (MCells[1] + 1) * (MCells[2] + 1) * (LocalIndexIn[0] - 1) + (MCells[2] + 1) * (LocalIndexIn[1] - 1) + LocalIndexIn[2] - 1;
		}

	}

	template<class T>
	void TMPMGrid<T>::UpdateGridFromPositions(const Chaos::TDynamicParticles<T, 3>& InParticles)
	{
		TVector<T, 3> ParticleMin, ParticleMax;
		if(InParticles.Size() > 0)
		{
			ParticleMin = InParticles.X()[0];
			ParticleMax = InParticles.X()[0];
			for(uint32 p = 0; p < InParticles.Size(); p++)
			{
				for(int32 alpha = 0; alpha < 3; alpha++)
				{
					if(ParticleMin[alpha] > InParticles.X()[p][alpha])
					{
						ParticleMin[alpha] = InParticles.X()[p][alpha];
					}
					if(ParticleMax[alpha] < InParticles.X()[p][alpha])
					{
						ParticleMax[alpha] = InParticles.X()[p][alpha];
					}
				}
			}
			TVector<T, 3> dims = ParticleMax - ParticleMin;

			for(uint32 c = 0; c < 3; c++)
			{
				MMinCorner[c] = MDx[c] * T(FMath::CeilToInt((ParticleMin[c] / MDx[c]))) - T(2) * MDx[c];
				MCells[c] = int32(FMath::CeilToInt((dims[c] / MDx[c]))) + 4;
				MMaxCorner[c] = MMinCorner[c] + T(MCells[c]) * MDx[c];
			}
		}
	}

	template<class T>
	void TMPMGrid<T>::SetInterp(InterpType InterpIn)
	{
		interp = InterpIn;
		if(interp == linear)
			NPerDir = 2;
		else
			NPerDir = 3;
	}



	template<class T>
	bool Chaos::TUniformGrid<T, 3>::IsValid(const TVector<int32, 3>& X) const
	{
		return X == ClampIndex(X);
	}
}

#if PLATFORM_MAC || PLATFORM_LINUX
template class CHAOS_API Chaos::TUniformGridBase<Chaos::FReal, 3>;
template class CHAOS_API Chaos::TUniformGrid<Chaos::FReal, 3>;
template class CHAOS_API Chaos::TUniformGrid<Chaos::FReal, 2>;
template class CHAOS_API Chaos::TMPMGrid<Chaos::FRealSingle>;
template class CHAOS_API Chaos::TMPMGrid<Chaos::FReal>;
#else
template class Chaos::TUniformGridBase<Chaos::FReal, 3>;
template class Chaos::TUniformGrid<Chaos::FReal, 3>;
template class Chaos::TUniformGrid<Chaos::FReal, 2>;
template class Chaos::TMPMGrid<Chaos::FRealSingle>;
template class Chaos::TMPMGrid<Chaos::FReal>;
#endif

template Chaos::FVec3 Chaos::TUniformGridBase<Chaos::FReal, 3>::LinearlyInterpolate<Chaos::FVec3>(const Chaos::TArrayND<Chaos::FVec3, 3>&, const Chaos::FVec3&) const;
template CHAOS_API Chaos::FReal Chaos::TUniformGridBase<Chaos::FReal, 3>::LinearlyInterpolate<Chaos::FReal>(const TArrayND<Chaos::FReal, 3>&, const Chaos::FVec3&) const;
