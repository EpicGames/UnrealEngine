// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/Utilities.h"
#include "Chaos/Matrix.h"
#include "Chaos/Rotation.h"
#include "Chaos/Framework/PhysicsSolverBase.h"

namespace Chaos
{
	namespace Utilities
	{
		FReal GetSolverPhysicsResultsTime(const FPhysicsSolverBase* Solver)
		{
			return Solver->GetPhysicsResultsTime_External();
		}

		// Use this to enable multiple Newton-Raphson steps rather than just a single one. However, note
		// that the benefit is tiny in normal cases - typically the first step achieves 95% of the convergence.
#define USE_MULTI_STEP_GYROSCOPIC_CALCULATION 0

		FVec3 GetAngularVelocityAdjustedForGyroscopicTorques(const FRotation3& Q, const FVec3& I, const FVec3& W, const FReal Dt)
		{
			if (Dt <= 0)
			{
				return W;
			}
			// See https://gdcvault.com/play/1022197
			//
			// The rotational motion is decomposed, and so we'll just apply the unforced part here by integrating:
			//
			// I WDot + W x I W = Torque
			//
			// Discretising WDot = (W2 - W1) / Dt
			//
			// and using W2 etc (i.e. solving implicitly) and only considering Torque = 0 
			//
			// I2 (W2 - W1) + Dt W2 x I2 W2 = 0
			//
			// We want to solve for W2 (noting that we don't have I2 yet).
			//
			// Newton-Raphson is used to solve problems in the form: f(x) = 0
			//
			// for a single var x, then iterate: x' = x - f(x) / f'(x)
			//
			// for multiple variables (X is a vector so call it x, y, z), the 1/f'(x) turns into multiply by
			// the inverse of the Jacobean where J = [df/dx, df/dy, df/dz]
			//
			// Then we have X' = X - JInv(X) * F(X)

			// Angular velocity in the space of the body, so that the inertia tensor is constant.
			FVec3 WBody = Q.UnrotateVector(W);

			const FMatrix33 IBody(I.X, 0, 0, 0, I.Y, 0, 0, 0, I.Z);

#if USE_MULTI_STEP_GYROSCOPIC_CALCULATION
			FVec3 WBodyOrig = WBody;
#endif

			// Note that FMatrix33::operator* applies from right to left, so the code here uses
			// Utilities::Multiply, which operates from left to right, matching normal notation.

#if USE_MULTI_STEP_GYROSCOPIC_CALCULATION
	// Newton-Raphson iteration
			for (int i = 0; i != 5; ++i)
#endif
			{
				// Evaluate F(X) - i.e. the terms which if re-evaluated after the solve we want to be zero
				// (because we are solving for the part where there is no external torque). Note that IBody
				// * (WBody - WBodyOrig) is always zero on the first step of Newton-Raphson.
				const FVec3 F =
#if USE_MULTI_STEP_GYROSCOPIC_CALCULATION
					Utilities::Multiply(IBody, (WBody - WBodyOrig)) +
#endif
					Dt * WBody.Cross(Utilities::Multiply(IBody, WBody));

				// Jacobian. We essentially differentiate F with respect to WBody, noting that IBody is constant.
				// The second term is differentiating a product, so D(uv) = D(u)v + uD(v)
				const FMatrix33 J = IBody +
					(Utilities::Multiply(Utilities::CrossProductMatrix(WBody), IBody) -
						Utilities::CrossProductMatrix(Utilities::Multiply(IBody, WBody))) * Dt;

				// It is more efficient to use a direct solve rather than calculating the inverse of J and
				// then multiplying.
				FVec3 JInvF;
				if (Utilities::Solve(JInvF, J, F)) // Equivalent to Utilities::Multiply(J.Inverse(), F);
				{
					WBody = WBody - JInvF;
				}
			}

			// Convert back to world coordinates
			return Q.RotateVector(WBody);

		}


	}
}
