// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/VelocityField.h"
#include "HAL/IConsoleManager.h"
#include "Chaos/GraphColoring.h"
#if INTEL_ISPC
#include "VelocityField.ispc.generated.h"

static_assert(sizeof(ispc::FVector3f) == sizeof(Chaos::Softs::FSolverVec3), "sizeof(ispc::FVector3f) != sizeof(Chaos::Softs::FSolverVec3)");
static_assert(sizeof(ispc::FVector2f) == sizeof(Chaos::Softs::FSolverVec2), "sizeof(ispc::FVector2f) != sizeof(Chaos::Softs::FSolverVec2)");
static_assert(sizeof(ispc::FIntVector) == sizeof(Chaos::TVec3<int32>), "sizeof(ispc::FIntVector) != sizeof(Chaos::TVec3<int32>)");
#endif

#if INTEL_ISPC && (!UE_BUILD_SHIPPING || USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING)
bool bChaos_VelocityField_ISPC_Enabled = CHAOS_VELOCITY_FIELD_ISPC_ENABLED_DEFAULT;
FAutoConsoleVariableRef CVarChaosVelocityFieldISPCEnabled(TEXT("p.Chaos.VelocityField.ISPC"), bChaos_VelocityField_ISPC_Enabled, TEXT("Whether to use ISPC optimizations in velocity field calculations"));
#endif

namespace Chaos::Softs {

namespace Private
{
	static float VelocityFieldMaxVelocity = 0.f;
	static FAutoConsoleVariableRef CVarChaosVelocityFieldMaxVelocity(TEXT("p.Chaos.VelocityField.MaxVelocity"), VelocityFieldMaxVelocity, TEXT("The maximum relative velocity to process the aerodynamics forces with."));
}

void FVelocityAndPressureField::SetProperties(
	const FCollectionPropertyConstFacade& PropertyCollection,
	const TMap<FString, TConstArrayView<FRealSingle>>& Weightmaps,
	FSolverReal WorldScale,
	bool bEnableAerodynamics)
{
	bool bSetMultipliers = false;

	if (IsDragMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedValue(GetWeightedFloatDrag(PropertyCollection));
		if (IsDragStringDirty(PropertyCollection))
		{
			const FString& WeightMapName = GetDragString(PropertyCollection);
			Drag = FPBDFlatWeightMap(
				WeightedValue.ClampAxes(MinCoefficient, MaxCoefficient),
				Weightmaps.FindRef(WeightMapName),
				TConstArrayView<TVec3<int32>>(Elements),
				Offset,
				NumParticles);

			if (OuterDragIndex == INDEX_NONE)
			{
				// OuterDrag is not specified, so it should match Drag.
				OuterDrag = Drag;
			}
		}
		else
		{
			Drag.SetWeightedValue(WeightedValue.ClampAxes(MinCoefficient, MaxCoefficient));
			if (OuterDragIndex == INDEX_NONE)
			{
				// OuterDrag is not specified, so it should match Drag.
				OuterDrag.SetWeightedValue(WeightedValue.ClampAxes(MinCoefficient, MaxCoefficient));
			}
		}
	}

	if (IsOuterDragMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedValue(GetWeightedFloatOuterDrag(PropertyCollection));
		if (IsOuterDragStringDirty(PropertyCollection))
		{
			const FString& WeightMapName = GetOuterDragString(PropertyCollection);
			OuterDrag = FPBDFlatWeightMap(
				WeightedValue.ClampAxes(MinCoefficient, MaxCoefficient),
				Weightmaps.FindRef(WeightMapName),
				TConstArrayView<TVec3<int32>>(Elements),
				Offset,
				NumParticles);
		}
		else
		{
			OuterDrag.SetWeightedValue(WeightedValue.ClampAxes(MinCoefficient, MaxCoefficient));
		}
	}

	if (IsLiftMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedValue(GetWeightedFloatLift(PropertyCollection));

		if (IsLiftStringDirty(PropertyCollection))
		{
			const FString& WeightMapName = GetLiftString(PropertyCollection);
			Lift = FPBDFlatWeightMap(
				WeightedValue.ClampAxes(MinCoefficient, MaxCoefficient),
				Weightmaps.FindRef(WeightMapName),
				TConstArrayView<TVec3<int32>>(Elements),
				Offset,
				NumParticles);

			if (OuterLiftIndex == INDEX_NONE)
			{
				// OuterLift is not specified, so it should match Lift.
				OuterLift = Lift;
			}
		}
		else
		{
			Lift.SetWeightedValue(WeightedValue.ClampAxes(MinCoefficient, MaxCoefficient));
			if (OuterLiftIndex == INDEX_NONE)
			{
				// OuterLift is not specified, so it should match Lift.
				OuterLift.SetWeightedValue(WeightedValue.ClampAxes(MinCoefficient, MaxCoefficient));
			}
		}
	}

	if (IsOuterLiftMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedValue(GetWeightedFloatOuterLift(PropertyCollection));

		if (IsOuterLiftStringDirty(PropertyCollection))
		{
			const FString& WeightMapName = GetOuterLiftString(PropertyCollection);
			OuterLift = FPBDFlatWeightMap(
				WeightedValue.ClampAxes(MinCoefficient, MaxCoefficient),
				Weightmaps.FindRef(WeightMapName),
				TConstArrayView<TVec3<int32>>(Elements),
				Offset,
				NumParticles);
		}
		else
		{
			OuterLift.SetWeightedValue(WeightedValue.ClampAxes(MinCoefficient, MaxCoefficient));
		}
	}

	if (IsFluidDensityMutable(PropertyCollection))
	{
		Rho = (FSolverReal)FMath::Max(GetFluidDensity(PropertyCollection), 0.f) / FMath::Cube(WorldScale);
	}

	if (IsTurbulenceRatioMutable(PropertyCollection))
	{
		TurbulenceRatio = FMath::Clamp(GetTurbulenceRatio(PropertyCollection), 0.f, 1.f);
	}

	if (IsPressureMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedValue(GetWeightedFloatPressure(PropertyCollection));
		if (IsPressureStringDirty(PropertyCollection))
		{
			const FString& WeightMapName = GetPressureString(PropertyCollection);
			Pressure = FPBDFlatWeightMap(
				WeightedValue / WorldScale,
				Weightmaps.FindRef(WeightMapName),
				TConstArrayView<TVec3<int32>>(Elements),
				Offset,
				NumParticles);
		}
		else
		{
			Pressure.SetWeightedValue(WeightedValue / WorldScale);
		}
	}

	// Update QuarterRho
	constexpr FSolverReal OneQuarter = (FSolverReal)0.25;
	QuarterRho = bEnableAerodynamics ? Rho * OneQuarter : (FSolverReal)0.;
}

void FVelocityAndPressureField::SetPropertiesAndWind(
	const FCollectionPropertyConstFacade& PropertyCollection,
	const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
	FSolverReal WorldScale,
	bool bEnableAerodynamics,
	const FSolverVec3& SolverWind,
	const FRotation3& LocalSpaceRotation,
	const FRotation3& ReferenceSpaceRotation)
{
	SetProperties(PropertyCollection, WeightMaps, WorldScale, bEnableAerodynamics);

	FSolverVec3 WindVelocity = WindVelocityIndex != INDEX_NONE ? WorldScale * FSolverVec3(GetWindVelocity(PropertyCollection)) : FSolverVec3(0.f);
	const EChaosSoftsSimulationSpace WindVelocitySpace = WindVelocitySpaceIndex != INDEX_NONE ? (EChaosSoftsSimulationSpace)GetWindVelocitySpace(PropertyCollection) : DefaultWindVelocitySpace;
	switch (WindVelocitySpace)
	{
	case EChaosSoftsSimulationSpace::WorldSpace:
	default:
		break;
	case EChaosSoftsSimulationSpace::ComponentSpace:
		WindVelocity = ((Softs::FSolverRotation3)LocalSpaceRotation).RotateVector(WindVelocity);
		break;
	case EChaosSoftsSimulationSpace::ReferenceBoneSpace:
		WindVelocity = ((Softs::FSolverRotation3)ReferenceSpaceRotation).RotateVector(WindVelocity);
		break;
	}
	SetVelocity(WindVelocity + SolverWind);
}

void FVelocityAndPressureField::SetProperties(
	const FSolverVec2& InDrag,
	const FSolverVec2& InOuterDrag,
	const FSolverVec2& InLift,
	const FSolverVec2& InOuterLift,
	const FSolverReal FluidDensity,
	const FSolverVec2& InPressure,
	FSolverReal WorldScale)
{
	Drag.SetWeightedValue(InDrag.ClampAxes(MinCoefficient, MaxCoefficient));
	OuterDrag.SetWeightedValue(InOuterDrag.ClampAxes(MinCoefficient, MaxCoefficient));
	Lift.SetWeightedValue(InLift.ClampAxes(MinCoefficient, MaxCoefficient));
	OuterLift.SetWeightedValue(InOuterLift.ClampAxes(MinCoefficient, MaxCoefficient));
	Pressure.SetWeightedValue(InPressure / WorldScale);
	Rho = FMath::Max(FluidDensity / FMath::Cube(WorldScale), (FSolverReal)0.);

	constexpr FSolverReal OneQuarter = (FSolverReal)0.25;
	QuarterRho = Rho * OneQuarter;
}

void FVelocityAndPressureField::SetGeometry(
	const FTriangleMesh* TriangleMesh,
	const FCollectionPropertyConstFacade& PropertyCollection,
	const TMap<FString, TConstArrayView<FRealSingle>>& Weightmaps,
	FSolverReal WorldScale)
{
	// Reinit indices
	DragIndex = FDragIndex(PropertyCollection);
	OuterDragIndex = FOuterDragIndex(PropertyCollection);
	LiftIndex = FLiftIndex(PropertyCollection);
	OuterLiftIndex = FOuterLiftIndex(PropertyCollection);
	FluidDensityIndex = FFluidDensityIndex(PropertyCollection);
	PressureIndex = FPressureIndex(PropertyCollection);
	TurbulenceRatioIndex = FTurbulenceRatioIndex(PropertyCollection);

	// Reset geometry, properties, and weight maps
	SetGeometry(TriangleMesh);
	// If these properties don't exist, set their values to 0, not to DefaultCoefficients!
	const FSolverVec2 DragValue = FSolverVec2(GetWeightedFloatDrag(PropertyCollection, 0.f));
	const FSolverVec2 LiftValue = FSolverVec2(GetWeightedFloatLift(PropertyCollection, 0.f));
	// OuterDrag and OuterLift default to Drag and Lift respectively.
	const FSolverVec2 OuterDragValue = FSolverVec2(GetWeightedFloatOuterDrag(PropertyCollection, DragValue));
	const FSolverVec2 OuterLiftValue = FSolverVec2(GetWeightedFloatOuterLift(PropertyCollection, LiftValue));
	SetProperties(
		DragValue,
		OuterDragValue,
		LiftValue,
		OuterLiftValue,
		(FSolverReal)GetFluidDensity(PropertyCollection, 0.f),
		FSolverVec2(GetWeightedFloatPressure(PropertyCollection, 0.f)),
		WorldScale);
	TurbulenceRatio = FMath::Clamp(GetTurbulenceRatio(PropertyCollection, DefaultTurbulenceRatio), 0.f, 1.f);
	SetMultipliers(PropertyCollection, Weightmaps);
	ResetColor();
}

void FVelocityAndPressureField::SetGeometry(
	const FTriangleMesh* TriangleMesh,
	const TConstArrayView<FRealSingle>& DragMultipliers,
	const TConstArrayView<FRealSingle>& OuterDragMultipliers,
	const TConstArrayView<FRealSingle>& LiftMultipliers,
	const TConstArrayView<FRealSingle>& OuterLiftMultipliers,
	const TConstArrayView<FRealSingle>& PressureMultipliers)
{
	SetGeometry(TriangleMesh);
	SetMultipliers(DragMultipliers, OuterDragMultipliers, LiftMultipliers, OuterLiftMultipliers, PressureMultipliers);
	ResetColor();
}


void FVelocityAndPressureField::InitColor(const FSolverParticlesRange& InParticles)
{
#if INTEL_ISPC
	const TArray<TArray<int32>> ConstraintsPerColor = FGraphColoring::ComputeGraphColoringParticlesOrRange(ElementsLocal, InParticles, Offset, Offset + NumParticles);
	TArray<TVec3<int32>> ReorderedElements;
	TArray<int32> OrigToReorderedIndices; // used to reorder stiffness indices
	ReorderedElements.SetNumUninitialized(ElementsLocal.Num());
	OrigToReorderedIndices.SetNumUninitialized(ElementsLocal.Num());
	ConstraintsPerColorStartIndex.Reset(ConstraintsPerColor.Num() + 1);
	int32 ReorderedIndex = 0;
	for (const TArray<int32>& ConstraintsBatch : ConstraintsPerColor)
	{
		ConstraintsPerColorStartIndex.Add(ReorderedIndex);
		for (const int32& BatchConstraint : ConstraintsBatch)
		{
			const int32 OrigIndex = BatchConstraint;
			ReorderedElements[ReorderedIndex] = ElementsLocal[OrigIndex];
			OrigToReorderedIndices[OrigIndex] = ReorderedIndex;
			++ReorderedIndex;
		}
	}
	ConstraintsPerColorStartIndex.Add(ReorderedIndex);

	ElementsLocal = MoveTemp(ReorderedElements);
	Elements = ElementsLocal; // Need to update pointer.
	Lift.ReorderIndices(OrigToReorderedIndices);
	Drag.ReorderIndices(OrigToReorderedIndices);
	Pressure.ReorderIndices(OrigToReorderedIndices);
	for (TArray<int32>& Elems : PointToTriangleMapLocal)
	{
		for (int32& Element : Elems)
		{
			Element = OrigToReorderedIndices[Element];
		}
	}
#else
	ResetColor();
#endif
}

void FVelocityAndPressureField::ResetColor()
{
	ConstraintsPerColorStartIndex.Reset();
}

void FVelocityAndPressureField::SetGeometry(const FSolverParticlesRange& Particles, const FTriangleMesh* TriangleMesh)
{
	if (TriangleMesh)
	{
		const TArray<TVec3<int32>>& InElements = TriangleMesh->GetElements();
		Offset = 0;
		NumParticles = Particles.Size();

		// Strip kinematic elements
		PointToTriangleMapLocal.Reset(NumParticles);
		PointToTriangleMapLocal.AddDefaulted(NumParticles);
		ElementsLocal.Reset(InElements.Num());
		for (const TVec3<int32>& Elem : InElements)
		{
			if (Particles.InvM(Elem[0]) != (FSolverReal)0.f || Particles.InvM(Elem[1]) != (FSolverReal)0.f || Particles.InvM(Elem[2]) != (FSolverReal)0.f)
			{
				const int32 ElemIndex = ElementsLocal.Add(Elem);
				PointToTriangleMapLocal[Elem[0] - Offset].Add(ElemIndex);
				PointToTriangleMapLocal[Elem[1] - Offset].Add(ElemIndex);
				PointToTriangleMapLocal[Elem[2] - Offset].Add(ElemIndex);
			}
		}
		Forces.SetNumUninitialized(ElementsLocal.Num());

		// Update views to point to local data.
		PointToTriangleMap = PointToTriangleMapLocal;
		Elements = ElementsLocal;
	}
	else
	{
		PointToTriangleMapLocal.Reset();
		PointToTriangleMap = TConstArrayView<TArray<int32>>();
		ElementsLocal.Reset();
		Elements = TConstArrayView<TVec3<int32>>();
		Offset = 0;
		NumParticles = 0;
		Forces.Reset();
	}
}

void FVelocityAndPressureField::SetGeometry(const FTriangleMesh* TriangleMesh)
{
	if (TriangleMesh)
	{
		PointToTriangleMap = TriangleMesh->GetPointToTriangleMap();
		PointToTriangleMapLocal.Reset();
		Elements = TriangleMesh->GetElements();
		ElementsLocal.Reset();
		const TVec2<int32> Range = TriangleMesh->GetVertexRange();
		Offset = Range[0];
		NumParticles = 1 + Range[1] - Offset;
		Forces.SetNumUninitialized(Elements.Num());
	}
	else
	{
		PointToTriangleMapLocal.Reset();
		PointToTriangleMap = TConstArrayView<TArray<int32>>();
		ElementsLocal.Reset();
		Elements = TConstArrayView<TVec3<int32>>();
		Offset = 0;
		NumParticles = 0;
		Forces.Reset();
	}
}

void FVelocityAndPressureField::SetMultipliers(
	const FCollectionPropertyConstFacade& PropertyCollection,
	const TMap<FString, TConstArrayView<FRealSingle>>& Weightmaps)
{
	const TConstArrayView<FRealSingle> DragMultipliers = (DragIndex != INDEX_NONE) ?
		Weightmaps.FindRef(GetDragString(PropertyCollection)) : TConstArrayView<FRealSingle>();
	const TConstArrayView<FRealSingle> OuterDragMultipliers = (OuterDragIndex != INDEX_NONE) ?
		Weightmaps.FindRef(GetOuterDragString(PropertyCollection)) : DragMultipliers; // OuterDrag defaults to Drag
	const TConstArrayView<FRealSingle> LiftMultipliers = (LiftIndex != INDEX_NONE) ?
		Weightmaps.FindRef(GetLiftString(PropertyCollection)) : TConstArrayView<FRealSingle>();
	const TConstArrayView<FRealSingle> OuterLiftMultipliers = (OuterLiftIndex != INDEX_NONE) ?
		Weightmaps.FindRef(GetOuterLiftString(PropertyCollection)) : LiftMultipliers; // OuterLift defaults to Lift
	const TConstArrayView<FRealSingle> PressureMultipliers = (PressureIndex != INDEX_NONE) ?
		Weightmaps.FindRef(GetPressureString(PropertyCollection)) : TConstArrayView<FRealSingle>();

	SetMultipliers(DragMultipliers, OuterDragMultipliers, LiftMultipliers, OuterLiftMultipliers, PressureMultipliers);
}

void FVelocityAndPressureField::SetMultipliers(
	const TConstArrayView<FRealSingle>& DragMultipliers,
	const TConstArrayView<FRealSingle>& OuterDragMultipliers,
	const TConstArrayView<FRealSingle>& LiftMultipliers,
	const TConstArrayView<FRealSingle>& OuterLiftMultipliers,
	const TConstArrayView<FRealSingle>& PressureMultipliers)
{	
	const FSolverVec2 DragValues(Drag.GetLow(), Drag.GetHigh());
	const FSolverVec2 OuterDragValues(OuterDrag.GetLow(), OuterDrag.GetHigh());
	const FSolverVec2 LiftValues(Lift.GetLow(), Lift.GetHigh());
	const FSolverVec2 OuterLiftValues(OuterLift.GetLow(), OuterLift.GetHigh());
	const FSolverVec2 PressureValues(Pressure.GetLow(), Pressure.GetHigh());
	Drag = FPBDFlatWeightMap(DragValues, DragMultipliers, TConstArrayView<TVec3<int32>>(Elements), Offset, NumParticles);
	OuterDrag = FPBDFlatWeightMap(OuterDragValues, OuterDragMultipliers, TConstArrayView<TVec3<int32>>(Elements), Offset, NumParticles);
	Lift = FPBDFlatWeightMap(LiftValues, LiftMultipliers, TConstArrayView<TVec3<int32>>(Elements), Offset, NumParticles);
	OuterLift = FPBDFlatWeightMap(OuterLiftValues, OuterLiftMultipliers, TConstArrayView<TVec3<int32>>(Elements), Offset, NumParticles);

	Pressure = FPBDFlatWeightMap(PressureValues, PressureMultipliers, TConstArrayView<TVec3<int32>>(Elements), Offset, NumParticles);
}

FSolverVec3 FVelocityAndPressureField::CalculateForce(const TConstArrayView<FSolverVec3>& Xs, const TConstArrayView<FSolverVec3>& Vs, int32 ElementIndex) const
{
	if (Private::VelocityFieldMaxVelocity <= 0.f)
	{
		return CalculateForce(Xs, Vs, ElementIndex, Velocity, Drag.GetValue(ElementIndex), OuterDrag.GetValue(ElementIndex), Lift.GetValue(ElementIndex), OuterLift.GetValue(ElementIndex), Pressure.GetValue(ElementIndex));
	}
	else
	{
		return CalculateForce(Xs, Vs, ElementIndex, Velocity, Drag.GetValue(ElementIndex), OuterDrag.GetValue(ElementIndex), Lift.GetValue(ElementIndex), OuterLift.GetValue(ElementIndex), Pressure.GetValue(ElementIndex), FMath::Square((FSolverReal)Private::VelocityFieldMaxVelocity));
	}
}

void FVelocityAndPressureField::UpdateForces(const FSolverParticles& InParticles, const FSolverReal /*Dt*/)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FVelocityAndPressureField_UpdateForces);
	const FSolverReal MaxVelocitySquared = (Private::VelocityFieldMaxVelocity > 0.f) ? FMath::Square((FSolverReal)Private::VelocityFieldMaxVelocity) : TNumericLimits<FSolverReal>::Max();

	const bool bDragHasMap = Drag.HasWeightMap();
	const bool bOuterDragHasMap = OuterDrag.HasWeightMap();
	const bool bLiftHasMap = Lift.HasWeightMap();
	const bool bOuterLiftHasMap = OuterLift.HasWeightMap();
	const bool bPressureHasMap = Pressure.HasWeightMap();
	if (!bDragHasMap && !bOuterDragHasMap && !bLiftHasMap && !bOuterLiftHasMap && !bPressureHasMap)
	{
#if INTEL_ISPC
		if (bRealTypeCompatibleWithISPC && bChaos_VelocityField_ISPC_Enabled)
		{
			if (MaxVelocitySquared == TNumericLimits<FSolverReal>::Max())
			{
				ispc::UpdateField(
					(ispc::FVector3f*)Forces.GetData(),
					(const ispc::FIntVector*)Elements.GetData(),
					(const ispc::FVector3f*)InParticles.GetV().GetData(),
					(const ispc::FVector3f*)InParticles.XArray().GetData(),
					(const ispc::FVector3f&)Velocity,
					QuarterRho,
					TurbulenceRatio,
					(FSolverReal)Drag,
					(FSolverReal)OuterDrag,
					(FSolverReal)Lift,
					(FSolverReal)OuterLift,
					(FSolverReal)Pressure,
					Elements.Num());
			}
			else
			{
				ispc::UpdateFieldAndClampVelocity(
					(ispc::FVector3f*)Forces.GetData(),
					(const ispc::FIntVector*)Elements.GetData(),
					(const ispc::FVector3f*)InParticles.GetV().GetData(),
					(const ispc::FVector3f*)InParticles.XArray().GetData(),
					(const ispc::FVector3f&)Velocity,
					QuarterRho,
					TurbulenceRatio,
					(FSolverReal)Drag,
					(FSolverReal)OuterDrag,
					(FSolverReal)Lift,
					(FSolverReal)OuterLift,
					(FSolverReal)Pressure,
					Elements.Num(),
					MaxVelocitySquared);
			}
		}
		else
#endif
		{
			if (MaxVelocitySquared == TNumericLimits<FSolverReal>::Max())
			{
				for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ++ElementIndex)
				{
					UpdateField(InParticles, ElementIndex, Velocity,
						(FSolverReal)Drag,
						(FSolverReal)OuterDrag,
						(FSolverReal)Lift,
						(FSolverReal)OuterLift,
						(FSolverReal)Pressure);
				}
			}
			else
			{
				for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ++ElementIndex)
				{
					UpdateField(InParticles, ElementIndex, Velocity,
						(FSolverReal)Drag,
						(FSolverReal)OuterDrag,
						(FSolverReal)Lift,
						(FSolverReal)OuterLift,
						(FSolverReal)Pressure, MaxVelocitySquared);
				}
			}
		}
	}
	else
	{
#if INTEL_ISPC
		if (bRealTypeCompatibleWithISPC && bChaos_VelocityField_ISPC_Enabled)
		{
			if (MaxVelocitySquared == TNumericLimits<FSolverReal>::Max())
			{
				ispc::UpdateFieldWithWeightMaps(
					(ispc::FVector3f*)Forces.GetData(),
					(const ispc::FIntVector*)Elements.GetData(),
					(const ispc::FVector3f*)InParticles.GetV().GetData(),
					(const ispc::FVector3f*)InParticles.XArray().GetData(),
					(const ispc::FVector3f&)Velocity,
					QuarterRho,
					TurbulenceRatio,
					bDragHasMap,
					reinterpret_cast<const ispc::FVector2f&>(Drag.GetOffsetRange()),
					bDragHasMap ? Drag.GetMapValues().GetData() : nullptr,
					bOuterDragHasMap,
					reinterpret_cast<const ispc::FVector2f&>(OuterDrag.GetOffsetRange()),
					bOuterDragHasMap ? OuterDrag.GetMapValues().GetData() : nullptr,
					bLiftHasMap,
					reinterpret_cast<const ispc::FVector2f&>(Lift.GetOffsetRange()),
					bLiftHasMap ? Lift.GetMapValues().GetData() : nullptr,
					bOuterLiftHasMap,
					reinterpret_cast<const ispc::FVector2f&>(OuterLift.GetOffsetRange()),
					bOuterLiftHasMap ? OuterLift.GetMapValues().GetData() : nullptr,
					bPressureHasMap,
					reinterpret_cast<const ispc::FVector2f&>(Pressure.GetOffsetRange()),
					bPressureHasMap ? Pressure.GetMapValues().GetData() : nullptr,
					Elements.Num());
			}
			else
			{
				ispc::UpdateFieldWithWeightMapsAndClampVelocity(
					(ispc::FVector3f*)Forces.GetData(),
					(const ispc::FIntVector*)Elements.GetData(),
					(const ispc::FVector3f*)InParticles.GetV().GetData(),
					(const ispc::FVector3f*)InParticles.XArray().GetData(),
					(const ispc::FVector3f&)Velocity,
					QuarterRho,
					TurbulenceRatio,
					bDragHasMap,
					reinterpret_cast<const ispc::FVector2f&>(Drag.GetOffsetRange()),
					bDragHasMap ? Drag.GetMapValues().GetData() : nullptr,
					bOuterDragHasMap,
					reinterpret_cast<const ispc::FVector2f&>(OuterDrag.GetOffsetRange()),
					bOuterDragHasMap ? OuterDrag.GetMapValues().GetData() : nullptr,
					bLiftHasMap,
					reinterpret_cast<const ispc::FVector2f&>(Lift.GetOffsetRange()),
					bLiftHasMap ? Lift.GetMapValues().GetData() : nullptr,
					bOuterLiftHasMap,
					reinterpret_cast<const ispc::FVector2f&>(OuterLift.GetOffsetRange()),
					bOuterLiftHasMap ? OuterLift.GetMapValues().GetData() : nullptr,
					bPressureHasMap,
					reinterpret_cast<const ispc::FVector2f&>(Pressure.GetOffsetRange()),
					bPressureHasMap ? Pressure.GetMapValues().GetData() : nullptr,
					Elements.Num(),
					MaxVelocitySquared);
			}
		}
		else
#endif
		{
			if (MaxVelocitySquared == TNumericLimits<FSolverReal>::Max())
			{
				for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ++ElementIndex)
				{
					const FSolverReal CdI = Drag.GetValue(ElementIndex);
					const FSolverReal CdO = OuterDrag.GetValue(ElementIndex);
					const FSolverReal ClI = Lift.GetValue(ElementIndex);
					const FSolverReal ClO = OuterLift.GetValue(ElementIndex);
					const FSolverReal Cp = Pressure.GetValue(ElementIndex);

					UpdateField(InParticles, ElementIndex, Velocity, CdI, CdO, ClI, ClO, Cp);
				}
			}
			else
			{
				for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ++ElementIndex)
				{
					const FSolverReal CdI = Drag.GetValue(ElementIndex);
					const FSolverReal CdO = OuterDrag.GetValue(ElementIndex);
					const FSolverReal ClI = Lift.GetValue(ElementIndex);
					const FSolverReal ClO = OuterLift.GetValue(ElementIndex);
					const FSolverReal Cp = Pressure.GetValue(ElementIndex);

					UpdateField(InParticles, ElementIndex, Velocity, CdI, CdO, ClI, ClO, Cp, MaxVelocitySquared);
				}
			}
		}
	}
}

void FVelocityAndPressureField::Apply(FSolverParticlesRange& InParticles, const FSolverReal Dt) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FVelocityAndPressureField_Apply);
	const FSolverReal MaxVelocitySquared = (Private::VelocityFieldMaxVelocity > 0.f) ? FMath::Square((FSolverReal)Private::VelocityFieldMaxVelocity) : TNumericLimits<FSolverReal>::Max();

	const bool bDragHasMap = Drag.HasWeightMap();
	const bool bOuterDragHasMap = OuterDrag.HasWeightMap();
	const bool bLiftHasMap = Lift.HasWeightMap();
	const bool bOuterLiftHasMap = OuterLift.HasWeightMap();
	const bool bPressureHasMap = Pressure.HasWeightMap();
	if (!bDragHasMap && !bOuterDragHasMap && !bLiftHasMap && !bOuterLiftHasMap && !bPressureHasMap)
	{
#if INTEL_ISPC
		if (bRealTypeCompatibleWithISPC && bChaos_VelocityField_ISPC_Enabled && ConstraintsPerColorStartIndex.Num() > 1)
		{
			const int32 ConstraintColorNum = ConstraintsPerColorStartIndex.Num() - 1;
			if (MaxVelocitySquared == TNumericLimits<FSolverReal>::Max())
			{
				for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
				{
					const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
					const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
					ispc::UpdateAndApplyVelocityField(
						(ispc::FVector3f*)InParticles.GetAcceleration().GetData(),
						InParticles.GetInvM().GetData(),
						(const ispc::FIntVector*)&Elements.GetData()[ColorStart],
						(const ispc::FVector3f*)InParticles.GetV().GetData(),
						(const ispc::FVector3f*)InParticles.XArray().GetData(),
						(const ispc::FVector3f&)Velocity,
						QuarterRho,
						TurbulenceRatio,
						(FSolverReal)Drag,
						(FSolverReal)OuterDrag,
						(FSolverReal)Lift,
						(FSolverReal)OuterLift,
						(FSolverReal)Pressure,
						ColorSize);
				}
			}
			else
			{
				for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
				{
					const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
					const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
					ispc::UpdateAndClampVelocityAndApplyVelocityField(
						(ispc::FVector3f*)InParticles.GetAcceleration().GetData(),
						InParticles.GetInvM().GetData(),
						(const ispc::FIntVector*)&Elements.GetData()[ColorStart],
						(const ispc::FVector3f*)InParticles.GetV().GetData(),
						(const ispc::FVector3f*)InParticles.XArray().GetData(),
						(const ispc::FVector3f&)Velocity,
						QuarterRho,
						TurbulenceRatio,
						(FSolverReal)Drag,
						(FSolverReal)OuterDrag,
						(FSolverReal)Lift,
						(FSolverReal)OuterLift,
						(FSolverReal)Pressure,
						ColorSize,
						MaxVelocitySquared);
				}
			}
		}
		else
#endif
		{
			if (MaxVelocitySquared == TNumericLimits<FSolverReal>::Max())
			{
				for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ++ElementIndex)
				{
					const FSolverVec3 Force = CalculateForce(InParticles.XArray(), InParticles.GetV(), ElementIndex, Velocity,
						(FSolverReal)Drag,
						(FSolverReal)OuterDrag,
						(FSolverReal)Lift,
						(FSolverReal)OuterLift,
						(FSolverReal)Pressure);
					InParticles.Acceleration(Elements[ElementIndex][0]) += InParticles.InvM(Elements[ElementIndex][0]) * Force;
					InParticles.Acceleration(Elements[ElementIndex][1]) += InParticles.InvM(Elements[ElementIndex][1]) * Force;
					InParticles.Acceleration(Elements[ElementIndex][2]) += InParticles.InvM(Elements[ElementIndex][2]) * Force;
				}
			}
			else
			{
				for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ++ElementIndex)
				{
					const FSolverVec3 Force = CalculateForce(InParticles.XArray(), InParticles.GetV(), ElementIndex, Velocity,
						(FSolverReal)Drag,
						(FSolverReal)OuterDrag,
						(FSolverReal)Lift,
						(FSolverReal)OuterLift,
						(FSolverReal)Pressure, MaxVelocitySquared);
					InParticles.Acceleration(Elements[ElementIndex][0]) += InParticles.InvM(Elements[ElementIndex][0]) * Force;
					InParticles.Acceleration(Elements[ElementIndex][1]) += InParticles.InvM(Elements[ElementIndex][1]) * Force;
					InParticles.Acceleration(Elements[ElementIndex][2]) += InParticles.InvM(Elements[ElementIndex][2]) * Force;
				}
			}
		}
	}
	else
	{
#if INTEL_ISPC
		if (bRealTypeCompatibleWithISPC && bChaos_VelocityField_ISPC_Enabled && ConstraintsPerColorStartIndex.Num() > 1)
		{
			const int32 ConstraintColorNum = ConstraintsPerColorStartIndex.Num() - 1;
			if (MaxVelocitySquared == TNumericLimits<FSolverReal>::Max())
			{
				for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
				{
					const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
					const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
					ispc::UpdateAndApplyVelocityFieldWithWeightMaps(
						(ispc::FVector3f*)InParticles.GetAcceleration().GetData(),
						InParticles.GetInvM().GetData(),
						(const ispc::FIntVector*)&Elements.GetData()[ColorStart],
						(const ispc::FVector3f*)InParticles.GetV().GetData(),
						(const ispc::FVector3f*)InParticles.XArray().GetData(),
						(const ispc::FVector3f&)Velocity,
						QuarterRho,
						TurbulenceRatio,
						bDragHasMap,
						reinterpret_cast<const ispc::FVector2f&>(Drag.GetOffsetRange()),
						bDragHasMap ? Drag.GetMapValues().GetData() : nullptr,
						bOuterDragHasMap,
						reinterpret_cast<const ispc::FVector2f&>(OuterDrag.GetOffsetRange()),
						bOuterDragHasMap ? OuterDrag.GetMapValues().GetData() : nullptr,
						bLiftHasMap,
						reinterpret_cast<const ispc::FVector2f&>(Lift.GetOffsetRange()),
						bLiftHasMap ? Lift.GetMapValues().GetData() : nullptr,
						bOuterLiftHasMap,
						reinterpret_cast<const ispc::FVector2f&>(OuterLift.GetOffsetRange()),
						bOuterLiftHasMap ? OuterLift.GetMapValues().GetData() : nullptr,
						bPressureHasMap,
						reinterpret_cast<const ispc::FVector2f&>(Pressure.GetOffsetRange()),
						bPressureHasMap ? Pressure.GetMapValues().GetData() : nullptr,
						ColorSize);
				}
			}
			else
			{
				for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
				{
					const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
					const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
					ispc::UpdateAndClampVelocityAndApplyVelocityFieldWithWeightMaps(
						(ispc::FVector3f*)InParticles.GetAcceleration().GetData(),
						InParticles.GetInvM().GetData(),
						(const ispc::FIntVector*)&Elements.GetData()[ColorStart],
						(const ispc::FVector3f*)InParticles.GetV().GetData(),
						(const ispc::FVector3f*)InParticles.XArray().GetData(),
						(const ispc::FVector3f&)Velocity,
						QuarterRho,
						TurbulenceRatio,
						bDragHasMap,
						reinterpret_cast<const ispc::FVector2f&>(Drag.GetOffsetRange()),
						bDragHasMap ? Drag.GetMapValues().GetData() : nullptr,
						bOuterDragHasMap,
						reinterpret_cast<const ispc::FVector2f&>(OuterDrag.GetOffsetRange()),
						bOuterDragHasMap ? OuterDrag.GetMapValues().GetData() : nullptr,
						bLiftHasMap,
						reinterpret_cast<const ispc::FVector2f&>(Lift.GetOffsetRange()),
						bLiftHasMap ? Lift.GetMapValues().GetData() : nullptr,
						bOuterLiftHasMap,
						reinterpret_cast<const ispc::FVector2f&>(OuterLift.GetOffsetRange()),
						bOuterLiftHasMap ? OuterLift.GetMapValues().GetData() : nullptr,
						bPressureHasMap,
						reinterpret_cast<const ispc::FVector2f&>(Pressure.GetOffsetRange()),
						bPressureHasMap ? Pressure.GetMapValues().GetData() : nullptr,
						ColorSize,
						MaxVelocitySquared);
				}
			}
		}
		else
#endif
		{
			if (MaxVelocitySquared == TNumericLimits<FSolverReal>::Max())
			{
				for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ++ElementIndex)
				{
					const FSolverReal CdI = Drag.GetValue(ElementIndex);
					const FSolverReal CdO = OuterDrag.GetValue(ElementIndex);
					const FSolverReal ClI = Lift.GetValue(ElementIndex);
					const FSolverReal ClO = OuterLift.GetValue(ElementIndex);
					const FSolverReal Cp = Pressure.GetValue(ElementIndex);

					const FSolverVec3 Force = CalculateForce(InParticles.XArray(), InParticles.GetV(), ElementIndex, Velocity, CdI, CdO, ClI, ClO, Cp);
					InParticles.Acceleration(Elements[ElementIndex][0]) += InParticles.InvM(Elements[ElementIndex][0]) * Force;
					InParticles.Acceleration(Elements[ElementIndex][1]) += InParticles.InvM(Elements[ElementIndex][1]) * Force;
					InParticles.Acceleration(Elements[ElementIndex][2]) += InParticles.InvM(Elements[ElementIndex][2]) * Force;
				}
			}
			else
			{
				for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ++ElementIndex)
				{
					const FSolverReal CdI = Drag.GetValue(ElementIndex);
					const FSolverReal CdO = OuterDrag.GetValue(ElementIndex);
					const FSolverReal ClI = Lift.GetValue(ElementIndex);
					const FSolverReal ClO = OuterLift.GetValue(ElementIndex);
					const FSolverReal Cp = Pressure.GetValue(ElementIndex);

					const FSolverVec3 Force = CalculateForce(InParticles.XArray(), InParticles.GetV(), ElementIndex, Velocity, CdI, CdO, ClI, ClO, Cp, MaxVelocitySquared);
					InParticles.Acceleration(Elements[ElementIndex][0]) += InParticles.InvM(Elements[ElementIndex][0]) * Force;
					InParticles.Acceleration(Elements[ElementIndex][1]) += InParticles.InvM(Elements[ElementIndex][1]) * Force;
					InParticles.Acceleration(Elements[ElementIndex][2]) += InParticles.InvM(Elements[ElementIndex][2]) * Force;
				}
			}
		}
	}
}

}  // End namespace Chaos::Softs
