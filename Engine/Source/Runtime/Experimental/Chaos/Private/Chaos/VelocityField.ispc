// Copyright Epic Games, Inc. All Rights Reserved.

#define EXPLICIT_VECTOR4 1

#include "Math/Vector.isph"

static const float OneThird = 1.0f/3.0f;

static inline float SafeNormalize(FVector3f &Direction)
{
	const float Size = sqrt(VectorSizeSquared(Direction));
	Direction = VectorSelect((Size < FLOAT_KINDA_SMALL_NUMBER), FloatForwardVector, Direction / Size);
	return (Size < FLOAT_KINDA_SMALL_NUMBER) ? 0 : Size;
}

export void UpdateField(uniform FVector3f Forces[],
								const uniform FIntVector Elements[],
								const uniform FVector3f V[],
								const uniform FVector3f X[],
								const uniform FVector3f& Velocity,
								const uniform float QuarterRho,
								const uniform float TurbulenceRatio,
								const uniform float CdI,
								const uniform float CdO,
								const uniform float ClI,
								const uniform float ClO,
								const uniform float Cp,
								const uniform int32 NumElements)
{
	varying FIntVector Element;
	uniform int Offset = 0;

#if HW_GATHER_SUPPORTED == 1
	if(programIndex < NumElements)
	{
		Element = VectorLoad(&Elements[Offset]);
	}
#endif

	foreach(ElementIndex = 0 ... NumElements)
	{
#if HW_GATHER_SUPPORTED == 0
		Element = VectorLoad(&Elements[Offset]);
#endif

		const FVector3f X0 = VectorGather(&X[Element.V[0]]);
		const FVector3f X1 = VectorGather(&X[Element.V[1]]);
		const FVector3f X2 = VectorGather(&X[Element.V[2]]);

		// Calculate the normal and the area of the surface exposed to the flow
		FVector3f N = VectorCross(X2 - X0, X1 - X0);
		const float DoubleArea = SafeNormalize(N);

		const FVector3f V0 = VectorGather(&V[Element.V[0]]);
		const FVector3f V1 = VectorGather(&V[Element.V[1]]);
		const FVector3f V2 = VectorGather(&V[Element.V[2]]);

		// Calculate the direction and the relative velocity of the triangle to the flow
		const FVector3f SurfaceVelocity = OneThird * (V0 + V1 + V2);
		const FVector3f RelVelocity = Velocity - SurfaceVelocity;

		// Set the aerodynamic forces
		const float VDotN = VectorDot(RelVelocity, N);
		const float VSquare = VectorDot(RelVelocity, RelVelocity);
		const float TurbulenceFactor = VSquare > FLOAT_SMALL_NUMBER ? ((1 - TurbulenceRatio) / sqrt(VSquare) + TurbulenceRatio) : 1;

#if HW_GATHER_SUPPORTED == 1
		if(ElementIndex + programCount < NumElements)
		{
			Element = VectorLoad(&Elements[Offset + programCount]);
		}
#endif

		const FVector3f EvenForce = (CdI - ClI) * VDotN * RelVelocity + ClI * VSquare * N;
		const FVector3f OddForce = (ClO - CdO) * VDotN * RelVelocity - ClO * VSquare * N;

		const FVector3f Force = QuarterRho * DoubleArea * TurbulenceFactor * VectorSelect(VDotN >= 0, EvenForce, OddForce) + DoubleArea * 0.5 * Cp * N;

		VectorStore(&Forces[Offset], Force);

		Offset += programCount;
	}
}

export void UpdateFieldWithWeightMaps(uniform FVector3f Forces[],
								const uniform FIntVector Elements[],
								const uniform FVector3f V[],
								const uniform FVector3f X[],
								const uniform FVector3f& Velocity,
								const uniform float QuarterRho,
								const uniform float TurbulenceRatio,
								const uniform bool bDragHasMap,
								const uniform FVector2f& DragOffsetRange,
								const uniform float DragMapValues[],
								const uniform bool bOuterDragHasMap,
								const uniform FVector2f& OuterDragOffsetRange,
								const uniform float OuterDragMapValues[],
								const uniform bool bLiftHasMap,
								const uniform FVector2f& LiftOffsetRange,
								const uniform float LiftMapValues[],
								const uniform bool bOuterLiftHasMap,
								const uniform FVector2f& OuterLiftOffsetRange,
								const uniform float OuterLiftMapValues[],
								const uniform bool bPressureHasMap,
								const uniform FVector2f& PressureOffsetRange,
								const uniform float PressureMapValues[],
								const uniform int32 NumElements)
{
	varying FIntVector Element;
	uniform int Offset = 0;

#if HW_GATHER_SUPPORTED == 1
	if(programIndex < NumElements)
	{
		Element = VectorLoad(&Elements[Offset]);
	}
#endif

	foreach(ElementIndex = 0 ... NumElements)
	{
#if HW_GATHER_SUPPORTED == 0
		Element = VectorLoad(&Elements[Offset]);
#endif
		const FVector3f X0 = VectorGather(&X[Element.V[0]]);
		const FVector3f X1 = VectorGather(&X[Element.V[1]]);
		const FVector3f X2 = VectorGather(&X[Element.V[2]]);

		// Calculate the normal and the area of the surface exposed to the flow
		FVector3f N = VectorCross(X2 - X0, X1 - X0);
		const float DoubleArea = SafeNormalize(N);

		const FVector3f V0 = VectorGather(&V[Element.V[0]]);
		const FVector3f V1 = VectorGather(&V[Element.V[1]]);
		const FVector3f V2 = VectorGather(&V[Element.V[2]]);

		// Calculate the direction and the relative velocity of the triangle to the flow
		const FVector3f SurfaceVelocity = OneThird * (V0 + V1 + V2);
		const FVector3f RelVelocity = Velocity - SurfaceVelocity;

		// Set the aerodynamic forces
		const float VDotN = VectorDot(RelVelocity, N);
		const float VSquare = VectorDot(RelVelocity, RelVelocity);
		const float TurbulenceFactor = VSquare > FLOAT_SMALL_NUMBER ? ((1 - TurbulenceRatio) / sqrt(VSquare) + TurbulenceRatio) : 1;

		const float CdI = bDragHasMap ? DragOffsetRange.V[0] + DragMapValues[ElementIndex] * DragOffsetRange.V[1] : DragOffsetRange.V[0];
		const float CdO = bOuterDragHasMap ? OuterDragOffsetRange.V[0] + OuterDragMapValues[ElementIndex] * OuterDragOffsetRange.V[1] : OuterDragOffsetRange.V[0];
		const float ClI = bLiftHasMap ? LiftOffsetRange.V[0] + LiftMapValues[ElementIndex] * LiftOffsetRange.V[1] : LiftOffsetRange.V[0];
		const float ClO = bOuterLiftHasMap ? OuterLiftOffsetRange.V[0] + OuterLiftMapValues[ElementIndex] * OuterLiftOffsetRange.V[1] : OuterLiftOffsetRange.V[0];
		const float Cp = bPressureHasMap ? PressureOffsetRange.V[0] + PressureMapValues[ElementIndex] * PressureOffsetRange.V[1] : PressureOffsetRange.V[0];

#if HW_GATHER_SUPPORTED == 1
		if(ElementIndex + programCount < NumElements)
		{
			Element = VectorLoad(&Elements[Offset + programCount]);
		}
#endif

		const FVector3f EvenForce = (CdI - ClI) * VDotN * RelVelocity + ClI * VSquare * N;
		const FVector3f OddForce = (ClO - CdO) * VDotN * RelVelocity - ClO * VSquare * N;
		
		const FVector3f Force = QuarterRho * DoubleArea * TurbulenceFactor * VectorSelect(VDotN >= 0, EvenForce, OddForce) + DoubleArea * 0.5 * Cp * N;

		VectorStore(&Forces[Offset], Force);

		Offset += programCount;
	}
}

export void UpdateFieldAndClampVelocity(uniform FVector3f Forces[],
								const uniform FIntVector Elements[],
								const uniform FVector3f V[],
								const uniform FVector3f X[],
								const uniform FVector3f& Velocity,
								const uniform float QuarterRho,
								const uniform float TurbulenceRatio,
								const uniform float CdI,
								const uniform float CdO,
								const uniform float ClI,
								const uniform float ClO,
								const uniform float Cp,
								const uniform int32 NumElements,
								const uniform float MaxVelocitySquared)
{
	varying FIntVector Element;
	uniform int Offset = 0;

#if HW_GATHER_SUPPORTED == 1
	if(programIndex < NumElements)
	{
		Element = VectorLoad(&Elements[Offset]);
	}
#endif

	foreach(ElementIndex = 0 ... NumElements)
	{
#if HW_GATHER_SUPPORTED == 0
		Element = VectorLoad(&Elements[Offset]);
#endif

		const FVector3f X0 = VectorGather(&X[Element.V[0]]);
		const FVector3f X1 = VectorGather(&X[Element.V[1]]);
		const FVector3f X2 = VectorGather(&X[Element.V[2]]);

		// Calculate the normal and the area of the surface exposed to the flow
		FVector3f N = VectorCross(X2 - X0, X1 - X0);
		const float DoubleArea = SafeNormalize(N);

		const FVector3f V0 = VectorGather(&V[Element.V[0]]);
		const FVector3f V1 = VectorGather(&V[Element.V[1]]);
		const FVector3f V2 = VectorGather(&V[Element.V[2]]);

		// Calculate the direction and the relative velocity of the triangle to the flow
		const FVector3f SurfaceVelocity = OneThird * (V0 + V1 + V2);
		FVector3f RelVelocity = Velocity - SurfaceVelocity;

		// Clamp the relative velocity
		const float RelVelocitySquared = VectorSizeSquared(RelVelocity);
		if (RelVelocitySquared > MaxVelocitySquared)
		{
			RelVelocity = RelVelocity * sqrt(MaxVelocitySquared / RelVelocitySquared);
		}

		// Set the aerodynamic forces
		const float VDotN = VectorDot(RelVelocity, N);
		const float VSquare = VectorDot(RelVelocity, RelVelocity);
		const float TurbulenceFactor = VSquare > FLOAT_SMALL_NUMBER ? ((1 - TurbulenceRatio) / sqrt(VSquare) + TurbulenceRatio) : 1;

#if HW_GATHER_SUPPORTED == 1
		if(ElementIndex + programCount < NumElements)
		{
			Element = VectorLoad(&Elements[Offset + programCount]);
		}
#endif

		const FVector3f EvenForce = (CdI - ClI) * VDotN * RelVelocity + ClI * VSquare * N;
		const FVector3f OddForce = (ClO - CdO) * VDotN * RelVelocity - ClO * VSquare * N;
		
		const FVector3f Force = QuarterRho * DoubleArea * TurbulenceFactor * VectorSelect(VDotN >= 0, EvenForce, OddForce) + DoubleArea * 0.5 * Cp * N;

		VectorStore(&Forces[Offset], Force);

		Offset += programCount;
	}
}

export void UpdateFieldWithWeightMapsAndClampVelocity(uniform FVector3f Forces[],
								const uniform FIntVector Elements[],
								const uniform FVector3f V[],
								const uniform FVector3f X[],
								const uniform FVector3f& Velocity,
								const uniform float QuarterRho,
								const uniform float TurbulenceRatio,
								const uniform bool bDragHasMap,
								const uniform FVector2f& DragOffsetRange,
								const uniform float DragMapValues[],
								const uniform bool bOuterDragHasMap,
								const uniform FVector2f& OuterDragOffsetRange,
								const uniform float OuterDragMapValues[],
								const uniform bool bLiftHasMap,
								const uniform FVector2f& LiftOffsetRange,
								const uniform float LiftMapValues[],
								const uniform bool bOuterLiftHasMap,
								const uniform FVector2f& OuterLiftOffsetRange,
								const uniform float OuterLiftMapValues[],
								const uniform bool bPressureHasMap,
								const uniform FVector2f& PressureOffsetRange,
								const uniform float PressureMapValues[],
								const uniform int32 NumElements,
								const uniform float MaxVelocitySquared)
{
	varying FIntVector Element;
	uniform int Offset = 0;

#if HW_GATHER_SUPPORTED == 1
	if(programIndex < NumElements)
	{
		Element = VectorLoad(&Elements[Offset]);
	}
#endif

	foreach(ElementIndex = 0 ... NumElements)
	{
#if HW_GATHER_SUPPORTED == 0
		Element = VectorLoad(&Elements[Offset]);
#endif
		const FVector3f X0 = VectorGather(&X[Element.V[0]]);
		const FVector3f X1 = VectorGather(&X[Element.V[1]]);
		const FVector3f X2 = VectorGather(&X[Element.V[2]]);

		// Calculate the normal and the area of the surface exposed to the flow
		FVector3f N = VectorCross(X2 - X0, X1 - X0);
		const float DoubleArea = SafeNormalize(N);

		const FVector3f V0 = VectorGather(&V[Element.V[0]]);
		const FVector3f V1 = VectorGather(&V[Element.V[1]]);
		const FVector3f V2 = VectorGather(&V[Element.V[2]]);

		// Calculate the direction and the relative velocity of the triangle to the flow
		const FVector3f SurfaceVelocity = OneThird * (V0 + V1 + V2);
		FVector3f RelVelocity = Velocity - SurfaceVelocity;

		// Clamp the relative velocity
		const float RelVelocitySquared = VectorSizeSquared(RelVelocity);
		if (RelVelocitySquared > MaxVelocitySquared)
		{
			RelVelocity = RelVelocity * sqrt(MaxVelocitySquared / RelVelocitySquared);
		}

		// Set the aerodynamic forces
		const float VDotN = VectorDot(RelVelocity, N);
		const float VSquare = VectorDot(RelVelocity, RelVelocity);
		const float TurbulenceFactor = VSquare > FLOAT_SMALL_NUMBER ? ((1 - TurbulenceRatio) / sqrt(VSquare) + TurbulenceRatio) : 1;
		
		const float CdI = bDragHasMap ? DragOffsetRange.V[0] + DragMapValues[ElementIndex] * DragOffsetRange.V[1] : DragOffsetRange.V[0];
		const float CdO = bOuterDragHasMap ? OuterDragOffsetRange.V[0] + OuterDragMapValues[ElementIndex] * OuterDragOffsetRange.V[1] : OuterDragOffsetRange.V[0];
		const float ClI = bLiftHasMap ? LiftOffsetRange.V[0] + LiftMapValues[ElementIndex] * LiftOffsetRange.V[1] : LiftOffsetRange.V[0];
		const float ClO = bOuterLiftHasMap ? OuterLiftOffsetRange.V[0] + OuterLiftMapValues[ElementIndex] * OuterLiftOffsetRange.V[1] : OuterLiftOffsetRange.V[0];
		const float Cp = bPressureHasMap ? PressureOffsetRange.V[0] + PressureMapValues[ElementIndex] * PressureOffsetRange.V[1] : PressureOffsetRange.V[0];

#if HW_GATHER_SUPPORTED == 1
		if(ElementIndex + programCount < NumElements)
		{
			Element = VectorLoad(&Elements[Offset + programCount]);
		}
#endif

		const FVector3f EvenForce = (CdI - ClI) * VDotN * RelVelocity + ClI * VSquare * N;
		const FVector3f OddForce = (ClO - CdO) * VDotN * RelVelocity - ClO * VSquare * N;
		
		const FVector3f Force = QuarterRho * DoubleArea * TurbulenceFactor * VectorSelect(VDotN >= 0, EvenForce, OddForce) + DoubleArea * 0.5 * Cp * N;

		VectorStore(&Forces[Offset], Force);

		Offset += programCount;
	}
}

export void UpdateAndApplyVelocityField(uniform FVector3f Acceleration[],
								const uniform float InvM[],
								const uniform FIntVector Elements[],
								const uniform FVector3f V[],
								const uniform FVector3f X[],
								const uniform FVector3f& Velocity,
								const uniform float QuarterRho,
								const uniform float TurbulenceRatio,
								const uniform float CdI,
								const uniform float CdO,
								const uniform float ClI,
								const uniform float ClO,
								const uniform float Cp,
								const uniform int32 NumElements)
{

	foreach(ElementIndex = 0 ... NumElements)
	{
		const FIntVector Element = VectorLoad(&Elements[extract(ElementIndex,0)]);
		const varying int32 Index0 = Element.V[0];
		const varying int32 Index1 = Element.V[1];
		const varying int32 Index2 = Element.V[2];

		const FVector3f X0 = VectorGather(&X[Index0]);
		const FVector3f X1 = VectorGather(&X[Index1]);
		const FVector3f X2 = VectorGather(&X[Index2]);

		// Calculate the normal and the area of the surface exposed to the flow
		FVector3f N = VectorCross(X2 - X0, X1 - X0);
		const float DoubleArea = SafeNormalize(N);

		const FVector3f V0 = VectorGather(&V[Index0]);
		const FVector3f V1 = VectorGather(&V[Index1]);
		const FVector3f V2 = VectorGather(&V[Index2]);

		// Calculate the direction and the relative velocity of the triangle to the flow
		const FVector3f SurfaceVelocity = OneThird * (V0 + V1 + V2);
		const FVector3f RelVelocity = Velocity - SurfaceVelocity;

		// Set the aerodynamic forces
		const float VDotN = VectorDot(RelVelocity, N);
		const float VSquare = VectorDot(RelVelocity, RelVelocity);
		const float TurbulenceFactor = VSquare > FLOAT_SMALL_NUMBER ? ((1 - TurbulenceRatio) / sqrt(VSquare) + TurbulenceRatio) : 1;
		
		const FVector3f EvenForce = (CdI - ClI) * VDotN * RelVelocity + ClI * VSquare * N;
		const FVector3f OddForce = (ClO - CdO) * VDotN * RelVelocity - ClO * VSquare * N;
		
		const FVector3f Force = QuarterRho * DoubleArea * TurbulenceFactor * VectorSelect(VDotN >= 0, EvenForce, OddForce) + DoubleArea * 0.5 * Cp * N;
		
		#pragma ignore warning(perf)
		const float InvM0 = InvM[Index0];
		#pragma ignore warning(perf)
		const float InvM1 = InvM[Index1];
		#pragma ignore warning(perf)
		const float InvM2 = InvM[Index2];

		const FVector3f Acceleration0 = VectorGather(&Acceleration[Index0]);
		const FVector3f Acceleration1 = VectorGather(&Acceleration[Index1]);
		const FVector3f Acceleration2 = VectorGather(&Acceleration[Index2]);

		VectorScatter(&Acceleration[Index0], Acceleration0 + Force * InvM0);
		VectorScatter(&Acceleration[Index1], Acceleration1 + Force * InvM1);
		VectorScatter(&Acceleration[Index2], Acceleration2 + Force * InvM2);
	}
}

export void UpdateAndClampVelocityAndApplyVelocityField(uniform FVector3f Acceleration[],
								const uniform float InvM[],
								const uniform FIntVector Elements[],
								const uniform FVector3f V[],
								const uniform FVector3f X[],
								const uniform FVector3f& Velocity,
								const uniform float QuarterRho,
								const uniform float TurbulenceRatio,
								const uniform float CdI,
								const uniform float CdO,
								const uniform float ClI,
								const uniform float ClO,
								const uniform float Cp,
								const uniform int32 NumElements,
								const uniform float MaxVelocitySquared)
{

	foreach(ElementIndex = 0 ... NumElements)
	{
		const FIntVector Element = VectorLoad(&Elements[extract(ElementIndex,0)]);
		const varying int32 Index0 = Element.V[0];
		const varying int32 Index1 = Element.V[1];
		const varying int32 Index2 = Element.V[2];

		const FVector3f X0 = VectorGather(&X[Index0]);
		const FVector3f X1 = VectorGather(&X[Index1]);
		const FVector3f X2 = VectorGather(&X[Index2]);

		// Calculate the normal and the area of the surface exposed to the flow
		FVector3f N = VectorCross(X2 - X0, X1 - X0);
		const float DoubleArea = SafeNormalize(N);

		const FVector3f V0 = VectorGather(&V[Index0]);
		const FVector3f V1 = VectorGather(&V[Index1]);
		const FVector3f V2 = VectorGather(&V[Index2]);

		// Calculate the direction and the relative velocity of the triangle to the flow
		const FVector3f SurfaceVelocity = OneThird * (V0 + V1 + V2);
		FVector3f RelVelocity = Velocity - SurfaceVelocity;

		// Clamp the relative velocity
		const float RelVelocitySquared = VectorSizeSquared(RelVelocity);
		if (RelVelocitySquared > MaxVelocitySquared)
		{
			RelVelocity = RelVelocity * sqrt(MaxVelocitySquared / RelVelocitySquared);
		}

		// Set the aerodynamic forces
		const float VDotN = VectorDot(RelVelocity, N);
		const float VSquare = VectorDot(RelVelocity, RelVelocity);
		
		const FVector3f EvenForce = (CdI - ClI) * VDotN * RelVelocity + ClI * VSquare * N;
		const FVector3f OddForce = (ClO - CdO) * VDotN * RelVelocity - ClO * VSquare * N;
		const float TurbulenceFactor = VSquare > FLOAT_SMALL_NUMBER ? ((1 - TurbulenceRatio) / sqrt(VSquare) + TurbulenceRatio) : 1;
		
		const FVector3f Force = QuarterRho * DoubleArea * TurbulenceFactor * VectorSelect(VDotN >= 0, EvenForce, OddForce) + DoubleArea * 0.5 * Cp * N;
		
		#pragma ignore warning(perf)
		const float InvM0 = InvM[Index0];
		#pragma ignore warning(perf)
		const float InvM1 = InvM[Index1];
		#pragma ignore warning(perf)
		const float InvM2 = InvM[Index2];

		const FVector3f Acceleration0 = VectorGather(&Acceleration[Index0]);
		const FVector3f Acceleration1 = VectorGather(&Acceleration[Index1]);
		const FVector3f Acceleration2 = VectorGather(&Acceleration[Index2]);

		VectorScatter(&Acceleration[Index0], Acceleration0 + Force * InvM0);
		VectorScatter(&Acceleration[Index1], Acceleration1 + Force * InvM1);
		VectorScatter(&Acceleration[Index2], Acceleration2 + Force * InvM2);
	}
}


export void UpdateAndApplyVelocityFieldWithWeightMaps(uniform FVector3f Acceleration[],
								const uniform float InvM[],
								const uniform FIntVector Elements[],
								const uniform FVector3f V[],
								const uniform FVector3f X[],
								const uniform FVector3f& Velocity,
								const uniform float QuarterRho,
								const uniform float TurbulenceRatio,
								const uniform bool bDragHasMap,
								const uniform FVector2f& DragOffsetRange,
								const uniform float DragMapValues[],
								const uniform bool bOuterDragHasMap,
								const uniform FVector2f& OuterDragOffsetRange,
								const uniform float OuterDragMapValues[],
								const uniform bool bLiftHasMap,
								const uniform FVector2f& LiftOffsetRange,
								const uniform float LiftMapValues[],
								const uniform bool bOuterLiftHasMap,
								const uniform FVector2f& OuterLiftOffsetRange,
								const uniform float OuterLiftMapValues[],
								const uniform bool bPressureHasMap,
								const uniform FVector2f& PressureOffsetRange,
								const uniform float PressureMapValues[],
								const uniform int32 NumElements)
{

	foreach(ElementIndex = 0 ... NumElements)
	{
		const FIntVector Element = VectorLoad(&Elements[extract(ElementIndex,0)]);
		const varying int32 Index0 = Element.V[0];
		const varying int32 Index1 = Element.V[1];
		const varying int32 Index2 = Element.V[2];

		const FVector3f X0 = VectorGather(&X[Index0]);
		const FVector3f X1 = VectorGather(&X[Index1]);
		const FVector3f X2 = VectorGather(&X[Index2]);

		// Calculate the normal and the area of the surface exposed to the flow
		FVector3f N = VectorCross(X2 - X0, X1 - X0);
		const float DoubleArea = SafeNormalize(N);

		const FVector3f V0 = VectorGather(&V[Index0]);
		const FVector3f V1 = VectorGather(&V[Index1]);
		const FVector3f V2 = VectorGather(&V[Index2]);

		// Calculate the direction and the relative velocity of the triangle to the flow
		const FVector3f SurfaceVelocity = OneThird * (V0 + V1 + V2);
		const FVector3f RelVelocity = Velocity - SurfaceVelocity;

		// Set the aerodynamic forces
		const float VDotN = VectorDot(RelVelocity, N);
		const float VSquare = VectorDot(RelVelocity, RelVelocity);
		const float TurbulenceFactor = VSquare > FLOAT_SMALL_NUMBER ? ((1 - TurbulenceRatio) / sqrt(VSquare) + TurbulenceRatio) : 1;
		
		const float CdI = bDragHasMap ? DragOffsetRange.V[0] + DragMapValues[ElementIndex] * DragOffsetRange.V[1] : DragOffsetRange.V[0];
		const float CdO = bOuterDragHasMap ? OuterDragOffsetRange.V[0] + OuterDragMapValues[ElementIndex] * OuterDragOffsetRange.V[1] : OuterDragOffsetRange.V[0];
		const float ClI = bLiftHasMap ? LiftOffsetRange.V[0] + LiftMapValues[ElementIndex] * LiftOffsetRange.V[1] : LiftOffsetRange.V[0];
		const float ClO = bOuterLiftHasMap ? OuterLiftOffsetRange.V[0] + OuterLiftMapValues[ElementIndex] * OuterLiftOffsetRange.V[1] : OuterLiftOffsetRange.V[0];
		const float Cp = bPressureHasMap ? PressureOffsetRange.V[0] + PressureMapValues[ElementIndex] * PressureOffsetRange.V[1] : PressureOffsetRange.V[0];
		
		const FVector3f EvenForce = (CdI - ClI) * VDotN * RelVelocity + ClI * VSquare * N;
		const FVector3f OddForce = (ClO - CdO) * VDotN * RelVelocity - ClO * VSquare * N;
		
		const FVector3f Force = QuarterRho * DoubleArea * TurbulenceFactor * VectorSelect(VDotN >= 0, EvenForce, OddForce) + DoubleArea * 0.5 * Cp * N;
		
		#pragma ignore warning(perf)
		const float InvM0 = InvM[Index0];
		#pragma ignore warning(perf)
		const float InvM1 = InvM[Index1];
		#pragma ignore warning(perf)
		const float InvM2 = InvM[Index2];

		const FVector3f Acceleration0 = VectorGather(&Acceleration[Index0]);
		const FVector3f Acceleration1 = VectorGather(&Acceleration[Index1]);
		const FVector3f Acceleration2 = VectorGather(&Acceleration[Index2]);

		VectorScatter(&Acceleration[Index0], Acceleration0 + Force * InvM0);
		VectorScatter(&Acceleration[Index1], Acceleration1 + Force * InvM1);
		VectorScatter(&Acceleration[Index2], Acceleration2 + Force * InvM2);
	}
}

export void UpdateAndClampVelocityAndApplyVelocityFieldWithWeightMaps(uniform FVector3f Acceleration[],
								const uniform float InvM[],
								const uniform FIntVector Elements[],
								const uniform FVector3f V[],
								const uniform FVector3f X[],
								const uniform FVector3f& Velocity,
								const uniform float QuarterRho,
								const uniform float TurbulenceRatio,
								const uniform bool bDragHasMap,
								const uniform FVector2f& DragOffsetRange,
								const uniform float DragMapValues[],
								const uniform bool bOuterDragHasMap,
								const uniform FVector2f& OuterDragOffsetRange,
								const uniform float OuterDragMapValues[],
								const uniform bool bLiftHasMap,
								const uniform FVector2f& LiftOffsetRange,
								const uniform float LiftMapValues[],
								const uniform bool bOuterLiftHasMap,
								const uniform FVector2f& OuterLiftOffsetRange,
								const uniform float OuterLiftMapValues[],
								const uniform bool bPressureHasMap,
								const uniform FVector2f& PressureOffsetRange,
								const uniform float PressureMapValues[],
								const uniform int32 NumElements,
								const uniform float MaxVelocitySquared)
{

	foreach(ElementIndex = 0 ... NumElements)
	{
		const FIntVector Element = VectorLoad(&Elements[extract(ElementIndex,0)]);
		const varying int32 Index0 = Element.V[0];
		const varying int32 Index1 = Element.V[1];
		const varying int32 Index2 = Element.V[2];

		const FVector3f X0 = VectorGather(&X[Index0]);
		const FVector3f X1 = VectorGather(&X[Index1]);
		const FVector3f X2 = VectorGather(&X[Index2]);

		// Calculate the normal and the area of the surface exposed to the flow
		FVector3f N = VectorCross(X2 - X0, X1 - X0);
		const float DoubleArea = SafeNormalize(N);

		const FVector3f V0 = VectorGather(&V[Index0]);
		const FVector3f V1 = VectorGather(&V[Index1]);
		const FVector3f V2 = VectorGather(&V[Index2]);

		// Calculate the direction and the relative velocity of the triangle to the flow
		const FVector3f SurfaceVelocity = OneThird * (V0 + V1 + V2);
		FVector3f RelVelocity = Velocity - SurfaceVelocity;

		// Clamp the relative velocity
		const float RelVelocitySquared = VectorSizeSquared(RelVelocity);
		if (RelVelocitySquared > MaxVelocitySquared)
		{
			RelVelocity = RelVelocity * sqrt(MaxVelocitySquared / RelVelocitySquared);
		}

		// Set the aerodynamic forces
		const float VDotN = VectorDot(RelVelocity, N);
		const float VSquare = VectorDot(RelVelocity, RelVelocity);
		const float TurbulenceFactor = VSquare > FLOAT_SMALL_NUMBER ? ((1 - TurbulenceRatio) / sqrt(VSquare) + TurbulenceRatio) : 1;
		
		const float CdI = bDragHasMap ? DragOffsetRange.V[0] + DragMapValues[ElementIndex] * DragOffsetRange.V[1] : DragOffsetRange.V[0];
		const float CdO = bOuterDragHasMap ? OuterDragOffsetRange.V[0] + OuterDragMapValues[ElementIndex] * OuterDragOffsetRange.V[1] : OuterDragOffsetRange.V[0];
		const float ClI = bLiftHasMap ? LiftOffsetRange.V[0] + LiftMapValues[ElementIndex] * LiftOffsetRange.V[1] : LiftOffsetRange.V[0];
		const float ClO = bOuterLiftHasMap ? OuterLiftOffsetRange.V[0] + OuterLiftMapValues[ElementIndex] * OuterLiftOffsetRange.V[1] : OuterLiftOffsetRange.V[0];
		const float Cp = bPressureHasMap ? PressureOffsetRange.V[0] + PressureMapValues[ElementIndex] * PressureOffsetRange.V[1] : PressureOffsetRange.V[0];
		
		const FVector3f EvenForce = (CdI - ClI) * VDotN * RelVelocity + ClI * VSquare * N;
		const FVector3f OddForce = (ClO - CdO) * VDotN * RelVelocity - ClO * VSquare * N;

		const FVector3f Force = QuarterRho * DoubleArea * TurbulenceFactor * VectorSelect(VDotN >= 0, EvenForce, OddForce) + DoubleArea * 0.5 * Cp * N;
		
		#pragma ignore warning(perf)
		const float InvM0 = InvM[Index0];
		#pragma ignore warning(perf)
		const float InvM1 = InvM[Index1];
		#pragma ignore warning(perf)
		const float InvM2 = InvM[Index2];

		const FVector3f Acceleration0 = VectorGather(&Acceleration[Index0]);
		const FVector3f Acceleration1 = VectorGather(&Acceleration[Index1]);
		const FVector3f Acceleration2 = VectorGather(&Acceleration[Index2]);

		VectorScatter(&Acceleration[Index0], Acceleration0 + Force * InvM0);
		VectorScatter(&Acceleration[Index1], Acceleration1 + Force * InvM1);
		VectorScatter(&Acceleration[Index2], Acceleration2 + Force * InvM2);
	}
}
