// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/WeightedLatticeImplicitObject.h"
#include "Chaos/Levelset.h"
#include "Chaos/Tetrahedron.h"
#include "Chaos/PBDSoftsSolverParticles.h"

namespace Chaos
{

static int32 Chaos_WeightedLattice_PhiAndNormalBatchSize = 16;
#if !UE_BUILD_SHIPPING
FAutoConsoleVariableRef CVarChaosWeightedLatticePhiAndNormalBatchSize(TEXT("p.Chaos.WeightedLattice.PhiAndNormal.ParallelBatchSize"), Chaos_WeightedLattice_PhiAndNormalBatchSize, TEXT("Parallel batch size PhiAndNormal queries"));
#endif

FWeightedLatticeImplicitObject::FEmbeddingCoordinate::FEmbeddingCoordinate(const TVec3<int32>& InCellIndex, const FVec3& TrilinearCoordinate)
	:CellIndex(InCellIndex)
{
	const FVec4 Coord4(TrilinearCoordinate.X, TrilinearCoordinate.Y, TrilinearCoordinate.Z, 1.);

	if (((CellIndex.X + CellIndex.Y + CellIndex.Z) & 1) == 0)
	{
		static const FVec4 EvenPlaneTests[4] =
		{
			FVec4(-1., 1., -1., 0.),
			FVec4(1., 1., 1., -2.),
			FVec4(1., -1., -1., 0.),
			FVec4(-1., -1., 1., 0.)
		};

		FReal PlaneTestValue = 0.;
		for (LocalTetrahedron = 0; LocalTetrahedron < 4; ++LocalTetrahedron)
		{
			PlaneTestValue = Dot4(EvenPlaneTests[LocalTetrahedron], Coord4);
			if (PlaneTestValue >= 0.)
			{
				break;
			}
		}
		switch (LocalTetrahedron)
		{
		case 0:
			BarycentricCoordinate = FVec3(PlaneTestValue, 1. - Coord4[1], Coord4[0]);
			break;
		case 1:
			BarycentricCoordinate = FVec3(PlaneTestValue, 1. - Coord4[2], 1. - Coord4[1]);
			break;
		case 2:
			BarycentricCoordinate = FVec3(PlaneTestValue, Coord4[2], Coord4[1]);
			break;
		case 3:
			BarycentricCoordinate = FVec3(PlaneTestValue, 1. - Coord4[2], Coord4[1]);
			break;
		case 4:
			BarycentricCoordinate = FVec3(-.5 * Dot4(EvenPlaneTests[3], Coord4), -.5 * Dot4(EvenPlaneTests[2], Coord4), -.5 * Dot4(EvenPlaneTests[1], Coord4));
			break;
		default:
			check(false);
		}
	}
	else
	{
		static const FVec4 OddPlaneTests[4] =
		{
			FVec4(-1., -1., -1., 1.),
			FVec4(-1., 1., 1., -1.),
			FVec4(1., -1., 1., -1.),
			FVec4(1., 1., -1., -1.)
		};
		FReal PlaneTestValue = 0.;
		for (LocalTetrahedron = 0; LocalTetrahedron < 4; ++LocalTetrahedron)
		{
			PlaneTestValue = Dot4(OddPlaneTests[LocalTetrahedron], Coord4);
			if (PlaneTestValue >= 0.)
			{
				break;
			}
		}
		switch (LocalTetrahedron)
		{
		case 0:
			BarycentricCoordinate = FVec3(PlaneTestValue, Coord4[0], Coord4[1]);
			break;
		case 1:
			BarycentricCoordinate = FVec3(PlaneTestValue, 1. - Coord4[2], Coord4[0]);
			break;
		case 2:
			BarycentricCoordinate = FVec3(PlaneTestValue, Coord4[1], 1. - Coord4[2]);
			break;
		case 3:
			BarycentricCoordinate = FVec3(PlaneTestValue, 1. - Coord4[0], 1. - Coord4[1]);
			break;
		case 4:
			BarycentricCoordinate = FVec3(-.5 * Dot4(OddPlaneTests[1], Coord4), -.5 * Dot4(OddPlaneTests[0], Coord4), -.5 * Dot4(OddPlaneTests[2], Coord4));
			break;
		default:
			check(false);
		}
	}
}

FMatrix FWeightedLatticeImplicitObject::FEmbeddingCoordinate::DeformationTransform(const TArrayND<FVec3, 3>& InDeformedPoints, const TUniformGrid<FReal, 3>& InGrid) const
{
	const TVec4<TVec3<int32>>& TetOffsets = TetrahedronOffsets();
	auto TestOffsets = [this, &TetOffsets, &InGrid, &InDeformedPoints](const FMatrix& Transform)
	{
		constexpr FReal Tolerance = 1e-3;
		for (int32 VertId = 0; VertId < 4; ++VertId)
		{
			const FVec3 Node = InGrid.Node(CellIndex + TetOffsets[VertId]);
			const FVec3 TransformedPoint = Transform.TransformPosition(Node);
			const FVec3 DeformedPoint = InDeformedPoints(CellIndex + TetOffsets[VertId]);
			const FReal DiffSq = (TransformedPoint - DeformedPoint).SquaredLength();
			check(DiffSq < Tolerance);
		}

	};

	if (LocalTetrahedron < 4)
	{
		// These are the trirectangular tetrahedra (angles at corner are right angles)
		const FVec3 Corner = InDeformedPoints(CellIndex + TetOffsets[0]);
		const FVec3 Axis1 = InDeformedPoints(CellIndex + TetOffsets[1]) - Corner;
		const FVec3 Axis2 = InDeformedPoints(CellIndex + TetOffsets[2]) - Corner;
		const FVec3 Axis3 = InDeformedPoints(CellIndex + TetOffsets[3]) - Corner;

		FMatrix Result;

		if (((CellIndex.X + CellIndex.Y + CellIndex.Z) & 1) == 0)
		{
			switch (LocalTetrahedron)
			{
			case 0:
				Result = FMatrix(Axis2 / InGrid.Dx().X, -Axis1 / InGrid.Dx().Y, Axis3 / InGrid.Dx().Z, FVec3(0, 0, 0));
				break;
			case 1:
				Result = FMatrix(-Axis3 / InGrid.Dx().X, -Axis2 / InGrid.Dx().Y, -Axis1 / InGrid.Dx().Z, FVec3(0, 0, 0));
				break;
			case 2:
				Result = FMatrix(-Axis3 / InGrid.Dx().X, Axis2 / InGrid.Dx().Y, Axis1 / InGrid.Dx().Z, FVec3(0, 0, 0));
				break;
			case 3:
				Result = FMatrix(Axis3 / InGrid.Dx().X, Axis2 / InGrid.Dx().Y, -Axis1 / InGrid.Dx().Z, FVec3(0, 0, 0));
				break;
			default:
				check(0);
			}
		}
		else
		{
			switch (LocalTetrahedron)
			{
			case 0:
				Result = FMatrix(Axis1 / InGrid.Dx().X, Axis2 / InGrid.Dx().Y, Axis3 / InGrid.Dx().Z, FVec3(0, 0, 0));
				break;
			case 1:
				Result = FMatrix(Axis2 / InGrid.Dx().X, -Axis3 / InGrid.Dx().Y, -Axis1 / InGrid.Dx().Z, FVec3(0, 0, 0));
				break;
			case 2:
				Result = FMatrix(-Axis3 / InGrid.Dx().X, Axis1 / InGrid.Dx().Y, -Axis2 / InGrid.Dx().Z, FVec3(0, 0, 0));
				break;
			case 3:
				Result = FMatrix(-Axis1 / InGrid.Dx().X, -Axis2 / InGrid.Dx().Y, Axis3 / InGrid.Dx().Z, FVec3(0, 0, 0));
				break;
			default:
				check(0);
			}
		}
		Result.SetOrigin(Corner - Result.TransformVector(InGrid.Node(CellIndex + TetOffsets[0])));
		TestOffsets(Result);
		return Result;
	}
	else
	{
		const FVec3& P0 = InDeformedPoints(CellIndex + TetOffsets[0]);
		const FVec3& P1 = InDeformedPoints(CellIndex + TetOffsets[1]);
		const FVec3& P2 = InDeformedPoints(CellIndex + TetOffsets[2]);
		const FVec3& P3 = InDeformedPoints(CellIndex + TetOffsets[3]);

		// These are the regular tetrahedra. 
		if (((CellIndex.X + CellIndex.Y + CellIndex.Z) & 1) == 0)
		{
			FMatrix Result(.5 * (P0 + P3 - P1 - P2) / InGrid.Dx().X, .5 * (P0 + P1 - P2 - P3) / InGrid.Dx().Y, .5 * (P1 + P3 - P0 - P2) / InGrid.Dx().Z, FVec3(0, 0, 0));
			Result.SetOrigin(P2 - Result.TransformVector(InGrid.Node(CellIndex + TetOffsets[2])));
			TestOffsets(Result);
			return Result;
		}
		else
		{
			FMatrix Result(.5 * (P0 + P1 - P2 - P3) / InGrid.Dx().X, .5 * (P1 + P2 - P0 - P3) / InGrid.Dx().Y, .5 * (P1 + P3 - P0 - P2) / InGrid.Dx().Z, FVec3(0, 0, 0));
			Result.SetOrigin(P1 - Result.TransformVector(InGrid.Node(CellIndex + TetOffsets[1])));
			TestOffsets(Result);
			return Result;
		}
	}
}

FWeightedLatticeImplicitObject::FWeightedLatticeImplicitObject(const FWeightedLatticeImplicitObject& Other)
	:FImplicitObject(EImplicitObject::HasBoundingBox, Other.GetType())
	, Grid(Other.Grid)
	, UsedBones(Other.UsedBones)
	, ReferenceRelativeTransforms(Other.ReferenceRelativeTransforms)
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	, SolverBoneIndices(Other.SolverBoneIndices)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	, LocalBoundingBox(Other.LocalBoundingBox)
{
	BoneData.Copy(Other.BoneData);
	DeformedPoints.Copy(Other.DeformedPoints);
	EmptyCells.Copy(Other.EmptyCells);
	if (!Other.bSpatialDirty)
	{
		UpdateSpatialHierarchy();
	}
	bSpatialDirty = Other.bSpatialDirty;
}

FWeightedLatticeImplicitObject::FWeightedLatticeImplicitObject(int32 Flags, EImplicitObjectType InType, TUniformGrid<FReal, 3>&& InGrid,
	TArrayND<FWeightedLatticeInfluenceData, 3>&& InBoneData, TArray<FName>&& InUsedBones, TArray<FTransform>&& InReferenceRelativeTransforms)
	: FImplicitObject(Flags, InType | ImplicitObjectType::IsWeightedLattice)
	, Grid(MoveTemp(InGrid))
	, BoneData(MoveTemp(InBoneData))
	, UsedBones(MoveTemp(InUsedBones))
	, ReferenceRelativeTransforms(MoveTemp(InReferenceRelativeTransforms))
{
	FinalizeConstruction();
}

FWeightedLatticeImplicitObject::FWeightedLatticeImplicitObject(FWeightedLatticeImplicitObject&& Other)
	: FImplicitObject(EImplicitObject::HasBoundingBox, Other.GetType())
	, Grid(MoveTemp(Other.Grid))
	, BoneData(MoveTemp(Other.BoneData))
	, UsedBones(MoveTemp(Other.UsedBones))
	, ReferenceRelativeTransforms(MoveTemp(Other.ReferenceRelativeTransforms))
	, EmptyCells(MoveTemp(Other.EmptyCells))
	, LocalBoundingBox(MoveTemp(Other.LocalBoundingBox))
	, DeformedPoints(MoveTemp(Other.DeformedPoints))
{
}

void FWeightedLatticeImplicitObject::FinalizeConstruction()
{
	SetEmptyCells();
	InitializeDeformedPoints();
	bSpatialDirty = true;
}

void FWeightedLatticeImplicitObject::Serialize(FChaosArchive& Ar)
{
	FImplicitObject::SerializeImp(Ar);
	Ar << Grid;
	Ar << BoneData;
	Ar << UsedBones;
	Ar << ReferenceRootTransform;
	Ar << ReferenceRelativeTransforms;
	if (Ar.IsLoading())
	{
		FinalizeConstruction();
	}
}

void FWeightedLatticeImplicitObject::DeformPoints(const TArray<FTransform>& RelativeTransforms)
{
	checkSlow(RelativeTransforms.Num() == ReferenceRelativeTransforms.Num());
	TArray<FTransform> BoneTransforms;
	BoneTransforms.SetNum(RelativeTransforms.Num());
	for (int32 Index = 0; Index < RelativeTransforms.Num(); ++Index)
	{
		BoneTransforms[Index] = ReferenceRelativeTransforms[Index] * RelativeTransforms[Index];
	}

	const TVec3<int32> NodeCounts = Grid.NodeCounts();
	LocalBoundingBox = TAABB<FReal, 3>::EmptyAABB();
	for (int32 I = 0; I < NodeCounts.X; ++I)
	{
		for (int32 J = 0; J < NodeCounts.Y; ++J)
		{
			for (int32 K = 0; K < NodeCounts.Z; ++K)
			{
				const TVec3<int32> Index(I, J, K);
				const FWeightedLatticeInfluenceData& InfluenceData = BoneData(Index);
				if (InfluenceData.NumInfluences == 0)
				{
					LocalBoundingBox.GrowToInclude(DeformedPoints(Index));
				}
				else
				{
					FVec3 NewPos(0.);
					const FVec3 UndeformedPos = Grid.Node(Index);
					for (int32 InfluenceIdx = 0; InfluenceIdx < InfluenceData.NumInfluences; ++InfluenceIdx)
					{
						NewPos += InfluenceData.BoneWeights[InfluenceIdx] * BoneTransforms[InfluenceData.BoneIndices[InfluenceIdx]].TransformPosition(UndeformedPos);
					}
					bSpatialDirty = FVec3::DistSquared(DeformedPoints(Index), NewPos) > UE_SMALL_NUMBER * UE_SMALL_NUMBER;
					DeformedPoints(Index) = NewPos;
					LocalBoundingBox.GrowToInclude(NewPos);
				}
			}
		}
	}
}

FVec3 FWeightedLatticeImplicitObject::GetDeformedPoint(const FVec3& UndeformedPoint) const
{
	const TVec3<int32> CellIndex = Grid.Cell(UndeformedPoint);
	if (EmptyCells(CellIndex))
	{
		return UndeformedPoint;
	}

	const FVec3 Alpha = ((UndeformedPoint - Grid.Node(CellIndex)) / Grid.Dx()).BoundToBox(FVec3(0.), FVec3(1.));
	FEmbeddingCoordinate EmbeddingCoord(CellIndex, Alpha);
	return EmbeddingCoord.DeformedPosition(DeformedPoints);
}

struct FWeightedLatticeBvEntry
{
	const FWeightedLatticeImplicitObject* Lattice;
	TVec3<int32> CellIndex;

	bool HasBoundingBox() const
	{
		return true;
	}

	Chaos::TAABB<FReal, 3> BoundingBox() const
	{
		Chaos::TAABB<FReal, 3> Bounds;
		for (int32 I = 0; I < 2; ++I)
		{
			for (int32 J = 0; J < 2; ++J)
			{
				for (int32 K = 0; K < 2; ++K)
				{
					Bounds.GrowToInclude(Lattice->GetDeformedPoints()(CellIndex + TVec3<int32>(I, J, K)));
				}
			}
		}
		return Bounds;
	}

	THierarchicalSpatialHash<int32, FReal>::FVectorAABB VectorAABB() const
	{
		THierarchicalSpatialHash<int32, FReal>::FVectorAABB Bounds(Lattice->GetDeformedPoints()(CellIndex));
		for (int32 I = 0; I < 2; ++I)
		{
			for (int32 J = 0; J < 2; ++J)
			{
				for (int32 K = 0; K < 2; ++K)
				{
					Bounds.GrowToInclude(Lattice->GetDeformedPoints()(CellIndex + TVec3<int32>(I, J, K)));
				}
			}
		}
		return Bounds;
	}

	template<typename TPayloadType>
	TPayloadType GetPayload(int32 Idx) const
	{
		return Lattice->GetGrid().FlatIndex(CellIndex, false);
	}
};

void FWeightedLatticeImplicitObject::UpdateSpatialHierarchy(const FReal MinLodSize)
{
	if (!bSpatialDirty)
	{
		return;
	}

	TArray<FWeightedLatticeBvEntry> Entries;
	Entries.Reserve(Grid.GetNumCells());
	for (int32 I = 0; I < Grid.Counts().X; ++I)
	{
		for (int32 J = 0; J < Grid.Counts().Y; ++J)
		{
			for (int32 K = 0; K < Grid.Counts().Z; ++K)
			{
				if (!EmptyCells(I, J, K))
				{
					Entries.Add({ this, {I,J,K} });
				}
			}
		}
	}

	Spatial.Initialize(Entries, MinLodSize);

	bSpatialDirty = false;
}

bool FWeightedLatticeImplicitObject::GetEmbeddingCoordinates(const FVec3& DeformedPoint, TArray<FEmbeddingCoordinate>& CoordinatesOut, bool bFindClosest) const
{
	check(!bSpatialDirty);
	CoordinatesOut.Reset();
	const bool bInLocalBoundingBox = LocalBoundingBox.Contains(DeformedPoint);
	if (!bInLocalBoundingBox && !bFindClosest)
	{
		return false;
	}

	TArray<int32> IntersectingCells = bInLocalBoundingBox ? Spatial.FindAllIntersections(DeformedPoint) : TArray<int32>();
	for (int32 IntersectingCell : IntersectingCells)
	{
		const TVec3<int32> CellIndex = Grid.GetIndex(IntersectingCell);
		const TVec4<TVec3<int32>>* TestCellOffsets = ((CellIndex.X + CellIndex.Y + CellIndex.Z) & 1) == 0 ? FEmbeddingCoordinate::EvenIndexTetrahedraOffsets : FEmbeddingCoordinate::OddIndexTetrahedraOffsets;
		for (int32 TestTetIndex = 0; TestTetIndex < 5; ++TestTetIndex)
		{
			const TTetrahedron<FReal> Tetrahedron(
				DeformedPoints(CellIndex + TestCellOffsets[TestTetIndex][0]),
				DeformedPoints(CellIndex + TestCellOffsets[TestTetIndex][1]),
				DeformedPoints(CellIndex + TestCellOffsets[TestTetIndex][2]),
				DeformedPoints(CellIndex + TestCellOffsets[TestTetIndex][3]));

			const TVec3<FReal> Weights = Tetrahedron.GetFirstThreeBarycentricCoordinates(DeformedPoint);
			constexpr FReal Tolerance = 0;
			const bool IsInside = Weights[0] >= Tolerance && Weights[1] >= Tolerance && Weights[2] >= Tolerance && Weights[0] + Weights[1] + Weights[2] <= 1 + Tolerance;
			if (IsInside)
			{
				CoordinatesOut.Emplace(CellIndex, TestTetIndex, Weights);
			}
		}
	}

	if (CoordinatesOut.Num() > 0 || !bFindClosest)
	{
		return CoordinatesOut.Num() > 0;
	}

	check(bFindClosest);
	FReal IntersectionTestRadius = 0.;
	
	// Keep increasing intersection test radius until find something.
	const FReal IntersectionTestRadiusBase = bInLocalBoundingBox ? 0. : LocalBoundingBox.SignedDistance(DeformedPoint);
	FReal IntersectionTestRadiusExpand= Grid.Dx().GetMax();
	constexpr int32 MaxIterativeQueries = 5;
	int32 IterativeQueryCount = 0;
	while (IntersectingCells.IsEmpty() && IterativeQueryCount < MaxIterativeQueries)
	{
		IntersectionTestRadiusExpand *= 2.;
		IntersectionTestRadius = IntersectionTestRadiusBase + IntersectionTestRadiusExpand;

		TAABB<FReal, 3> QueryBounds(DeformedPoint, DeformedPoint);
		QueryBounds.Thicken(IntersectionTestRadius);

		IntersectingCells = Spatial.FindAllIntersections(QueryBounds);	
		++IterativeQueryCount;
	}

	const bool bDoBruteForceQuery = IntersectingCells.IsEmpty() && IterativeQueryCount == MaxIterativeQueries;
	if (bDoBruteForceQuery)
	{
		const int32 NumCells = Grid.GetNumCells();
		IntersectingCells.Reserve(NumCells);
		for (int32 FlatCellIndex = 0; FlatCellIndex < NumCells; ++FlatCellIndex)
		{
			if (!EmptyCells[FlatCellIndex])
			{
				IntersectingCells.Add(FlatCellIndex);
			}
		}
	}

	FEmbeddingCoordinate ClosestPointCoord;
	FReal ClosestPointDistSq = std::numeric_limits<FReal>::max();
	for (int32 IntersectingCell : IntersectingCells)
	{
		const TVec3<int32> CellIndex = Grid.GetIndex(IntersectingCell);
		const TAABB<FReal, 3> BBox = FWeightedLatticeBvEntry({ this, CellIndex }).BoundingBox();
		if (FMath::Square(BBox.SignedDistance(DeformedPoint)) > ClosestPointDistSq)
		{
			continue;
		}
		const TVec4<TVec3<int32>>* TestCellOffsets = ((CellIndex.X + CellIndex.Y + CellIndex.Z) & 1) == 0 ? FEmbeddingCoordinate::EvenIndexTetrahedraOffsets : FEmbeddingCoordinate::OddIndexTetrahedraOffsets;
		for (int32 TestTetIndex = 0; TestTetIndex < 5; ++TestTetIndex)
		{
			const TTetrahedron<FReal> Tetrahedron(
				DeformedPoints(CellIndex + TestCellOffsets[TestTetIndex][0]),
				DeformedPoints(CellIndex + TestCellOffsets[TestTetIndex][1]),
				DeformedPoints(CellIndex + TestCellOffsets[TestTetIndex][2]),
				DeformedPoints(CellIndex + TestCellOffsets[TestTetIndex][3]));

			TVec4<FReal> Barycentric;
			const FVec3 ClosestPoint = Tetrahedron.FindClosestPointAndBary(DeformedPoint, Barycentric);
			const FReal DistSq = FVec3::DistSquared(ClosestPoint, DeformedPoint);
			if (DistSq < ClosestPointDistSq)
			{
				ClosestPointDistSq = DistSq;
				ClosestPointCoord = FEmbeddingCoordinate(CellIndex, TestTetIndex, Barycentric);
			}
		}
	}

	check(ClosestPointCoord.IsValid());

	// If closest point is farther than last search radius, need to do an expanded search to see if we missed anything.
	if (!bDoBruteForceQuery && ClosestPointDistSq > FMath::Square(IntersectionTestRadius))
	{
		TAABB<FReal, 3> QueryBounds(DeformedPoint, DeformedPoint);
		QueryBounds.Thicken(FMath::Sqrt(ClosestPointDistSq));

		const TArray<int32> FinalIntersectingCells = Spatial.FindAllIntersections(QueryBounds);
		const TSet<int32> OrigIntersectingCellsSet(IntersectingCells);
		for (int32 IntersectingCell : FinalIntersectingCells)
		{
			if (OrigIntersectingCellsSet.Contains(IntersectingCell))
			{
				// This cell was tested in the last pass.
				continue;
			}

			const TVec3<int32> CellIndex = Grid.GetIndex(IntersectingCell);
			const TAABB<FReal, 3> BBox = FWeightedLatticeBvEntry({ this, CellIndex }).BoundingBox();
			if (FMath::Square(BBox.SignedDistance(DeformedPoint)) > ClosestPointDistSq)
			{
				continue;
			}
			const TVec4<TVec3<int32>>* TestCellOffsets = ((CellIndex.X + CellIndex.Y + CellIndex.Z) & 1) == 0 ? FEmbeddingCoordinate::EvenIndexTetrahedraOffsets : FEmbeddingCoordinate::OddIndexTetrahedraOffsets;
			for (int32 TestTetIndex = 0; TestTetIndex < 5; ++TestTetIndex)
			{
				const TTetrahedron<FReal> Tetrahedron(
					DeformedPoints(CellIndex + TestCellOffsets[TestTetIndex][0]),
					DeformedPoints(CellIndex + TestCellOffsets[TestTetIndex][1]),
					DeformedPoints(CellIndex + TestCellOffsets[TestTetIndex][2]),
					DeformedPoints(CellIndex + TestCellOffsets[TestTetIndex][3]));

				TVec4<FReal> Barycentric;
				const FVec3 ClosestPoint = Tetrahedron.FindClosestPointAndBary(DeformedPoint, Barycentric);
				const FReal DistSq = FVec3::DistSquared(ClosestPoint, DeformedPoint);
				if (DistSq < ClosestPointDistSq)
				{
					ClosestPointDistSq = DistSq;
					ClosestPointCoord = FEmbeddingCoordinate(CellIndex, TestTetIndex, Barycentric);
				}
			}
		}
	}
	check(ClosestPointCoord.IsValid());
	CoordinatesOut.Add(ClosestPointCoord);
	return true;
}

void FWeightedLatticeImplicitObject::InitializeDeformedPoints()
{
	constexpr bool NodeValues = true;
	DeformedPoints.SetCounts(Grid, NodeValues);
	const TVec3<int32> NodeCounts = Grid.NodeCounts();
	for (int32 I = 0; I < NodeCounts.X; ++I)
	{
		for (int32 J = 0; J < NodeCounts.Y; ++J)
		{
			for (int32 K = 0; K < NodeCounts.Z; ++K)
			{
				DeformedPoints(I, J, K) = Grid.Node(TVec3<int32>(I, J, K));
			}
		}
	}
	LocalBoundingBox = TAABB<FReal, 3>(Grid.MinCorner(), Grid.MaxCorner());
}

void FWeightedLatticeImplicitObject::SetEmptyCells()
{
	constexpr bool NodeValues = false;
	EmptyCells.SetCounts(Grid, NodeValues);
	const TVec3<int32> CellCounts = Grid.Counts();
	for (int32 I = 0; I < CellCounts.X; ++I)
	{
		for (int32 J = 0; J < CellCounts.Y; ++J)
		{
			for (int32 K = 0; K < CellCounts.Z; ++K)
			{
				// Cell is empty if any of its 8 corners have zero weights
				const bool IsEmpty = (BoneData(I, J, K).NumInfluences == 0 ||
					BoneData(I, J, K + 1).NumInfluences == 0 ||
					BoneData(I, J + 1, K).NumInfluences == 0 ||
					BoneData(I, J + 1, K + 1).NumInfluences == 0 ||
					BoneData(I + 1, J, K).NumInfluences == 0 ||
					BoneData(I + 1, J, K + 1).NumInfluences == 0 ||
					BoneData(I + 1, J + 1, K).NumInfluences == 0 ||
					BoneData(I + 1, J + 1, K + 1).NumInfluences == 0);
				EmptyCells(I, J, K) = IsEmpty;
			}
		}
	}
}

uint32 FWeightedLatticeImplicitObject::GetTypeHashHelper(const uint32 InHash) const
{
	uint32 Result = InHash;

	// TypeHash BoneData
	TConstArrayView< FWeightedLatticeInfluenceData > BoneDataFlattened(BoneData.GetData(), BoneData.Num());
	for (const FWeightedLatticeInfluenceData& Data : BoneDataFlattened)
	{
		Result = HashCombine(Result, Data.GetTypeHash());
	}

	// TypeHash DeformedPoints
	TConstArrayView<FVec3> DeformedPointsFlattened(DeformedPoints.GetData(), DeformedPoints.Num());
	for (const FVec3& Point : DeformedPointsFlattened)
	{
		Result = HashCombine(Result, HashCombine(::GetTypeHash(Point.X), HashCombine(::GetTypeHash(Point.Y), ::GetTypeHash(Point.Z))));
	}
	return Result;
}

template<typename TConcrete>
TWeightedLatticeImplicitObject<TConcrete>::TWeightedLatticeImplicitObject(ObjectType&& InObject, TUniformGrid<FReal, 3>&& InGrid,
	TArrayND<FWeightedLatticeInfluenceData, 3>&& InBoneData, TArray<FName>&& InUsedBones, TArray<FTransform>&& InReferenceRelativeTransforms)
	: FWeightedLatticeImplicitObject(EImplicitObject::HasBoundingBox, InObject->GetType(), MoveTemp(InGrid), MoveTemp(InBoneData), MoveTemp(InUsedBones), MoveTemp(InReferenceRelativeTransforms))
	, Object(MoveTemp(InObject))
{
	this->bDoCollide = Object->GetDoCollide();
}

template<typename TConcrete>
TWeightedLatticeImplicitObject<TConcrete>::TWeightedLatticeImplicitObject(TWeightedLatticeImplicitObject&& Other)
	: FWeightedLatticeImplicitObject(MoveTemp(Other))
	, Object(MoveTemp(Other.Object))
{
	this->bDoCollide = Other.Object->GetDoCollide();
}

template<typename TConcrete>
FImplicitObjectPtr TWeightedLatticeImplicitObject<TConcrete>::CopyGeometry() const
{
	if (Object)
	{
		FImplicitObjectPtr CopiedShape = Object->CopyGeometry();
		return new TWeightedLatticeImplicitObject<TConcrete>(reinterpret_cast<ObjectType&&>(CopiedShape), *this);
	}
	else
	{
		check(false);
		return nullptr;
	}
}

template<typename TConcrete>
FImplicitObjectPtr TWeightedLatticeImplicitObject<TConcrete>::DeepCopyGeometry() const
{
	if (Object)
	{
		FImplicitObjectPtr CopiedObject = Object->DeepCopyGeometry();
		FImplicitObjectPtr NewObj(new TWeightedLatticeImplicitObject(reinterpret_cast<ObjectType&&>(CopiedObject), *this));
		return NewObj;
	}
	else
	{
		check(false);
		return nullptr;
	}
}

template<typename TConcrete>
FReal TWeightedLatticeImplicitObject<TConcrete>::PhiWithNormalAndSurfacePoint(const FVec3& X, FVec3& Normal, FEmbeddingCoordinate& SurfaceCoord, bool bIncludeEmptyCells) const
{
	TArray<FEmbeddingCoordinate> Coordinates;
	if (GetEmbeddingCoordinates(X, Coordinates, bIncludeEmptyCells))
	{
		// Choose "deepest" point here.
		FReal MinPhiUndeformed = std::numeric_limits<FReal>::max();
		int32 MinCoordIndex = -1;
		FVec3 MinNormalUndeformed;
		for (int32 CoordIndex = 0; CoordIndex < Coordinates.Num(); ++CoordIndex)
		{
			FVec3 UndeformedNormal;
			const FReal Phi = Object->PhiWithNormal(Coordinates[CoordIndex].UndeformedPosition(Grid), UndeformedNormal);
			if (Phi < MinPhiUndeformed)
			{
				MinPhiUndeformed = Phi;
				MinCoordIndex = CoordIndex;
				MinNormalUndeformed = UndeformedNormal;
			}
		}
		const FEmbeddingCoordinate& Coordinate = Coordinates[MinCoordIndex];

		const FVec3 UndeformedPoint = Coordinate.UndeformedPosition(Grid);
		const FVec3 UndeformedSurfacePoint = UndeformedPoint - MinPhiUndeformed * MinNormalUndeformed;

		const TVec3<int32> SurfaceCell = Grid.Cell(UndeformedSurfacePoint);
		const FVec3 SurfaceAlpha = ((UndeformedSurfacePoint - Grid.Node(SurfaceCell)) / Grid.Dx()).BoundToBox(FVec3(0.), FVec3(1.));
		SurfaceCoord = FEmbeddingCoordinate(SurfaceCell, SurfaceAlpha);

		const FVec3 DeformedSurfacePoint = SurfaceCoord.DeformedPosition(DeformedPoints);
		Normal = X - DeformedSurfacePoint;
		const FReal NormalLen = Normal.Length();
		const FReal Phi = MinPhiUndeformed < 0 ? -NormalLen : NormalLen;

		if (NormalLen < UE_SMALL_NUMBER)
		{
			const FMatrix Transform = Coordinate.DeformationTransform(DeformedPoints, Grid);
			Normal = Transform.TransformVector(MinNormalUndeformed.GetSafeNormal());
			Normal.SafeNormalize();
		}
		else
		{
			Normal /= Phi;
		}

		return Phi;
	}
	else
	{
		SurfaceCoord = FEmbeddingCoordinate();
		return UE_BIG_NUMBER;
	}
}

template<typename TConcrete>
void TWeightedLatticeImplicitObject<TConcrete>::BatchPhiWithNormalAndGreatestInfluenceBone(const TConstArrayView<Softs::FPAndInvM> PAndInvMArray, 
	const Softs::FSolverRigidTransform3& SolverToThis, const Softs::FSolverReal Thickness,
	TArray<Softs::FSolverReal>& OutBatchPhis, TArray<Softs::FSolverVec3>& OutBatchNormals, TArray<int32>& OutBatchGreatestInfluenceSolverBones) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(TWeightedLatticeImplicitObject_BatchPhiWithNormalAndGreatestInfluenceBone);
	const int32 NumParticles = PAndInvMArray.Num();
	OutBatchPhis.SetNumUninitialized(NumParticles);
	OutBatchNormals.SetNumUninitialized(NumParticles);
	OutBatchGreatestInfluenceSolverBones.SetNumUninitialized(NumParticles);
	
	const Softs::FPAndInvM* const PAndInvM = PAndInvMArray.GetData();
	Softs::FSolverReal* const Phis = OutBatchPhis.GetData();
	Softs::FSolverVec3* const Normals = OutBatchNormals.GetData();
	int32* const GreatestInfluenceSolverBones = OutBatchGreatestInfluenceSolverBones.GetData();

	const int32 NumBatches = FMath::CeilToInt((FRealSingle)(NumParticles) / (FRealSingle)Chaos_WeightedLattice_PhiAndNormalBatchSize);
	PhysicsParallelFor(NumBatches, [this, NumParticles, &PAndInvM, &SolverToThis, Thickness, &Phis, &Normals, &GreatestInfluenceSolverBones](int32 BatchNumber)
	{
		const int32 BatchBegin = (Chaos_WeightedLattice_PhiAndNormalBatchSize * BatchNumber);
		const int32 BatchEnd = FMath::Min(NumParticles, BatchBegin + Chaos_WeightedLattice_PhiAndNormalBatchSize);
		for (int32 Index = BatchBegin; Index < BatchEnd; ++Index)
		{
			if (PAndInvM[Index].InvM == (Softs::FSolverReal)0.)
			{
				continue;
			}

			const FVec3 RigidSpacePosition(SolverToThis.TransformPositionNoScale(PAndInvM[Index].P));
			FWeightedLatticeImplicitObject::FEmbeddingCoordinate SurfaceCoord;
			FVec3 Normal;
			constexpr bool bIncludeEmptyCellsFalse = false;
			Phis[Index] = (Softs::FSolverReal)PhiWithNormalAndSurfacePoint(RigidSpacePosition, Normal, SurfaceCoord, bIncludeEmptyCellsFalse);
			Normals[Index] = Softs::FSolverVec3(Normal);
			const Softs::FSolverReal Penetration = Thickness - Phis[Index]; // This is related to the Normal impulse
			GreatestInfluenceSolverBones[Index] = Penetration > (Softs::FSolverReal)0. ? SurfaceCoord.GreatestInfluenceBone(GetBoneData()) : INDEX_NONE;
		}
	});
}

template<typename TConcrete>
void TWeightedLatticeImplicitObject<TConcrete>::BatchPhiWithNormal(const TConstArrayView<Softs::FPAndInvM> PAndInvMArray,
	const Softs::FSolverRigidTransform3& SolverToThis, TArray<Softs::FSolverReal>& OutBatchPhis, TArray<Softs::FSolverVec3>& OutBatchNormals) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(TWeightedLatticeImplicitObject_BatchPhiWithNormal);
	const int32 NumParticles = PAndInvMArray.Num();
	OutBatchPhis.SetNumUninitialized(NumParticles);
	OutBatchNormals.SetNumUninitialized(NumParticles);
	const Softs::FPAndInvM* const PAndInvM = PAndInvMArray.GetData();
	Softs::FSolverReal* const Phis = OutBatchPhis.GetData();
	Softs::FSolverVec3* const Normals = OutBatchNormals.GetData();

	const int32 NumBatches = FMath::CeilToInt((FRealSingle)(NumParticles) / (FRealSingle)Chaos_WeightedLattice_PhiAndNormalBatchSize);
	PhysicsParallelFor(NumBatches, [this, NumParticles, &PAndInvM, &SolverToThis, &Phis, &Normals](int32 BatchNumber)
	{
		const int32 BatchBegin = (Chaos_WeightedLattice_PhiAndNormalBatchSize * BatchNumber);
		const int32 BatchEnd = FMath::Min(NumParticles, BatchBegin + Chaos_WeightedLattice_PhiAndNormalBatchSize);
		for (int32 Index = BatchBegin; Index < BatchEnd; ++Index)
		{
			if (PAndInvM[Index].InvM == (Softs::FSolverReal)0.)
			{
				continue;
			}
			const FVec3 RigidSpacePosition(SolverToThis.TransformPositionNoScale(PAndInvM[Index].P));
			FWeightedLatticeImplicitObject::FEmbeddingCoordinate SurfaceCoord;
			FVec3 Normal;
			constexpr bool bIncludeEmptyCellsFalse = false;
			Phis[Index] = (Softs::FSolverReal)PhiWithNormalAndSurfacePoint(RigidSpacePosition, Normal, SurfaceCoord, bIncludeEmptyCellsFalse);
			Normals[Index] = Softs::FSolverVec3(Normal);
		}
	});
}

void FWeightedLatticeImplicitObjectBuilder::GenerateGrid(const int32 GridResolution, const TAABB<FReal, 3>& ObjectBBox)
{
	constexpr int32 NumGhostCells = 3;
	const int32 DesiredGridResolutionNoPad = FMath::Max(GridResolution - 2 * NumGhostCells, 1); // We'll add ghost cells to grid. 
	const FVec3 Extents = ObjectBBox.Extents();
	const FReal Dx = Extents.GetMax() / DesiredGridResolutionNoPad;
	TVec3<int32> Cells(FMath::CeilToInt32(Extents.X / Dx), FMath::CeilToInt32(Extents.Y / Dx), FMath::CeilToInt32(Extents.Z / Dx));
	Cells = Cells.ComponentwiseMax(TVec3<int32>(1, 1, 1));
	const FVec3 MinCornerAdjusted = ObjectBBox.Min() - Dx * FVec3(1.) * .5; // Want bbox corners to be at cell centers.
	const FVec3 MaxCornerAdjusted = ObjectBBox.Min() + FVec3((FReal)Cells.X + .5, (FReal)Cells.Y, (FReal)Cells.Z) * Dx;
	Grid = TUniformGrid<FReal, 3>(ObjectBBox.Min(), MaxCornerAdjusted, Cells, NumGhostCells);

	BoneData.Reset();
	BoneData.SetCounts(Grid, true);
	FMemory::Memzero(BoneData.GetData(), sizeof(FWeightedLatticeInfluenceData) * BoneData.Num());

	BuildData.Reset();
	BuildData.SetNum(BoneData.Num());
	BuildStep = EBuildStep::GridValid;
}

void FWeightedLatticeImplicitObjectBuilder::AddInfluence(int32 FlatIndex, uint16 BoneIndex, float Weight, bool bIsOuterWeight)
{
	// Keep influences sorted by weight (highest to lowest)
	check(BuildStep == EBuildStep::GridValid);
	FWeightedLatticeInfluenceData& Data = BoneData[FlatIndex];
	FInfluenceBuildData& Build = BuildData[FlatIndex];

	if (bIsOuterWeight && !Build.WeightsAreOuter)
	{
		// Inner/surface weights take precedence over outer weights.
		return;
	}
	if (!bIsOuterWeight && Build.WeightsAreOuter)
	{
		// Clear all existing outer weights
		Data.NumInfluences = 0;
	}

	uint8 InsertIndex = Data.NumInfluences;
	for (uint8 Idx = 0; Idx < Data.NumInfluences; ++Idx)
	{
		if (Weight > Data.BoneWeights[Idx])
		{
			InsertIndex = Idx;
			break;
		}
	}

	if (InsertIndex == Data.NumInfluences && Data.NumInfluences == Data.MaxTotalInfluences)
	{
		return;
	}

	if (Data.NumInfluences < Data.MaxTotalInfluences)
	{
		++Data.NumInfluences;
	}

	for (int32 Idx = Data.NumInfluences - 1; Idx > (int32)InsertIndex; --Idx)
	{
		check(Idx < Data.MaxTotalInfluences);
		check(Idx > 0);
		Data.BoneIndices[Idx] = Data.BoneIndices[Idx - 1];
		Data.BoneWeights[Idx] = Data.BoneWeights[Idx - 1];
	}

	check(InsertIndex < Data.MaxTotalInfluences);
	Data.BoneIndices[InsertIndex] = BoneIndex;
	Data.BoneWeights[InsertIndex] = Weight;
	Build.WeightsAreOuter = bIsOuterWeight;
}

void FWeightedLatticeImplicitObjectBuilder::NormalizeBoneWeights()
{
	for (int32 Idx = 0; Idx < BoneData.Num(); ++Idx)
	{
		FWeightedLatticeInfluenceData& Data = BoneData[Idx];
		float TotalWeight = 0.f;
		for (uint8 InfIdx = 0; InfIdx < Data.NumInfluences; ++InfIdx)
		{
			TotalWeight += Data.BoneWeights[InfIdx];
		}
		constexpr float MIN_TOTAL_OUTER_WEIGHT = 0.1f;
		if(TotalWeight < UE_SMALL_NUMBER ||
			(BuildData[Idx].WeightsAreOuter && TotalWeight < MIN_TOTAL_OUTER_WEIGHT))
		{
			Data.NumInfluences = 0;
		}
		for (uint8 InfIdx = 0; InfIdx < Data.NumInfluences; ++InfIdx)
		{
			Data.BoneWeights[InfIdx] /= TotalWeight;
		}
	}
}

// Explicit instantiations
template class CHAOS_API TWeightedLatticeImplicitObject<FLevelSet>;
} // namespace Chaos
