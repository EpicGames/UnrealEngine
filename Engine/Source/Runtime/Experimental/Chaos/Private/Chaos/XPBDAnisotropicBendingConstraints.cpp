// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/XPBDAnisotropicBendingConstraints.h"
#include "Chaos/XPBDBendingConstraints.h"
#include "Chaos/GraphColoring.h"
#include "Chaos/Framework/Parallel.h"
#include "Chaos/TriangleMesh.h"
#include "ChaosStats.h"

#if INTEL_ISPC
#include "XPBDBendingConstraints.ispc.generated.h"
#include "XPBDAnisotropicBendingConstraints.ispc.generated.h"
#endif

DECLARE_CYCLE_STAT(TEXT("Chaos XPBD Anisotropic Bending Constraint"), STAT_XPBD_AnisoBending, STATGROUP_Chaos);

#if INTEL_ISPC && !UE_BUILD_SHIPPING
static_assert(sizeof(ispc::FVector4f) == sizeof(Chaos::Softs::FPAndInvM), "sizeof(ispc::FVector4f) != sizeof(Chaos::Softs::FPAndInvM");
static_assert(sizeof(ispc::FVector3f) == sizeof(Chaos::Softs::FSolverVec3), "sizeof(ispc::FVector3f) != sizeof(Chaos::Softs::FSolverVec3");
static_assert(sizeof(ispc::FVector2f) == sizeof(Chaos::Softs::FSolverVec2), "sizeof(ispc::FVector2f) != sizeof(Chaos::Softs::FSolverVec2");
#endif

namespace Chaos::Softs {

// @todo(chaos): the parallel threshold (or decision to run parallel) should probably be owned by the solver and passed to the constraint container
extern int32 Chaos_XPBDBending_ParallelConstraintCount;

static int32 Chaos_XPBDBending_ISPC_ParallelBatchSize = 1028;
static int32 Chaos_XPBDBending_ISPC_MinNumParallelBatches = 1028;  // effectively disabled for now
FAutoConsoleVariableRef CVarChaosXPBDBendingISPCParallelBatchSize(TEXT("p.Chaos.XPBDBending.ISPC.ParallelBatchSize"), Chaos_XPBDBending_ISPC_ParallelBatchSize, TEXT("Parallel batch size for ISPC XPBDBending constraints"));
FAutoConsoleVariableRef CVarChaosXPBDBendingISPCMinNumParallelBatches(TEXT("p.Chaos.XPBDBending.ISPC.MinNumParallelBatches"), Chaos_XPBDBending_ISPC_MinNumParallelBatches, TEXT("Min number of batches to invoke parallelFor ISPC XPBDBending constraints"));

FXPBDAnisotropicBendingConstraints::FXPBDAnisotropicBendingConstraints(const FSolverParticlesRange& InParticles,
	const FTriangleMesh& TriangleMesh,
	const TArray<TVec3<FVec2f>>& FaceVertexPatternPositions,
	const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
	const FCollectionPropertyConstFacade& PropertyCollection)
	: Base(
		InParticles,
		TriangleMesh.GetUniqueAdjacentElements(),
		TConstArrayView<FRealSingle>(), // We don't use base stiffness weight maps
		WeightMaps.FindRef(GetXPBDAnisoBucklingRatioString(PropertyCollection, XPBDAnisoBucklingRatioName.ToString())),
		TConstArrayView<FRealSingle>(), // We don't use base stiffness weight maps
		GetRestAngleMapFromCollection(WeightMaps, PropertyCollection),
		FSolverVec2(GetWeightedFloatXPBDAnisoBendingStiffnessWarp(PropertyCollection, MaxStiffness)),
		FSolverVec2(GetWeightedFloatXPBDAnisoBucklingRatio(PropertyCollection, 0.f)),
		FSolverVec2(GetWeightedFloatXPBDAnisoBucklingStiffnessWarp(PropertyCollection, MaxStiffness)),
		GetRestAngleValueFromCollection(PropertyCollection),
		(ERestAngleConstructionType)GetXPBDAnisoRestAngleType(PropertyCollection, (int32)ERestAngleConstructionType::Use3DRestAngles),
		true /*bTrimKinematicConstraints*/,
		MaxStiffness)
	, StiffnessWarp(
		FSolverVec2(GetWeightedFloatXPBDAnisoBendingStiffnessWarp(PropertyCollection, MaxStiffness)).ClampAxes(0, MaxStiffness),
		WeightMaps.FindRef(GetXPBDAnisoBendingStiffnessWarpString(PropertyCollection, XPBDAnisoBendingStiffnessWarpName.ToString())),
		TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
		ParticleOffset,
		ParticleCount)
	, StiffnessWeft(
		FSolverVec2(GetWeightedFloatXPBDAnisoBendingStiffnessWeft(PropertyCollection, MaxStiffness)).ClampAxes(0, MaxStiffness),
		WeightMaps.FindRef(GetXPBDAnisoBendingStiffnessWeftString(PropertyCollection, XPBDAnisoBendingStiffnessWeftName.ToString())),
		TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
		ParticleOffset,
		ParticleCount)
	, StiffnessBias(
		FSolverVec2(GetWeightedFloatXPBDAnisoBendingStiffnessBias(PropertyCollection, MaxStiffness)).ClampAxes(0, MaxStiffness),
		WeightMaps.FindRef(GetXPBDAnisoBendingStiffnessBiasString(PropertyCollection, XPBDAnisoBendingStiffnessBiasName.ToString())),
		TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
		ParticleOffset,
		ParticleCount)
	, BucklingStiffnessWarp(
		FSolverVec2(GetWeightedFloatXPBDAnisoBucklingStiffnessWarp(PropertyCollection, MaxStiffness)).ClampAxes(0, MaxStiffness),
		WeightMaps.FindRef(GetXPBDAnisoBucklingStiffnessWarpString(PropertyCollection, XPBDAnisoBucklingStiffnessWarpName.ToString())),
		TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
		ParticleOffset,
		ParticleCount)
	, BucklingStiffnessWeft(
		FSolverVec2(GetWeightedFloatXPBDAnisoBucklingStiffnessWeft(PropertyCollection, MaxStiffness)).ClampAxes(0, MaxStiffness),
		WeightMaps.FindRef(GetXPBDAnisoBucklingStiffnessWeftString(PropertyCollection, XPBDAnisoBucklingStiffnessWeftName.ToString())),
		TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
		ParticleOffset,
		ParticleCount)
	, BucklingStiffnessBias(
		FSolverVec2(GetWeightedFloatXPBDAnisoBucklingStiffnessBias(PropertyCollection, MaxStiffness)).ClampAxes(0, MaxStiffness),
		WeightMaps.FindRef(GetXPBDAnisoBucklingStiffnessBiasString(PropertyCollection, XPBDAnisoBucklingStiffnessBiasName.ToString())),
		TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
		ParticleOffset,
		ParticleCount)
	, DampingRatio(
		FSolverVec2(GetWeightedFloatXPBDAnisoBendingDamping(PropertyCollection, MinDamping)).ClampAxes(MinDamping, MaxDamping),
		WeightMaps.FindRef(GetXPBDAnisoBendingDampingString(PropertyCollection, XPBDAnisoBendingDampingName.ToString())),
		TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
		ParticleOffset,
		ParticleCount)
	, WarpWeftBiasBaseMultipliers(GenerateWarpWeftBiasBaseMultipliers(FaceVertexPatternPositions, TriangleMesh))
	, XPBDAnisoBendingStiffnessWarpIndex(PropertyCollection)
	, XPBDAnisoBendingStiffnessWeftIndex(PropertyCollection)
	, XPBDAnisoBendingStiffnessBiasIndex(PropertyCollection)
	, XPBDAnisoBendingDampingIndex(PropertyCollection)
	, XPBDAnisoBucklingRatioIndex(PropertyCollection)
	, XPBDAnisoBucklingStiffnessWarpIndex(PropertyCollection)
	, XPBDAnisoBucklingStiffnessWeftIndex(PropertyCollection)
	, XPBDAnisoBucklingStiffnessBiasIndex(PropertyCollection)
	, XPBDAnisoFlatnessRatioIndex(PropertyCollection)
	, XPBDAnisoRestAngleIndex(PropertyCollection)
	, XPBDAnisoRestAngleTypeIndex(PropertyCollection)
{
	Lambdas.Init((FSolverReal)0., Constraints.Num());
	InitColor(InParticles);
}

FXPBDAnisotropicBendingConstraints::FXPBDAnisotropicBendingConstraints(const FSolverParticles& InParticles,
	int32 InParticleOffset,
	int32 InParticleCount,
	const FTriangleMesh& TriangleMesh,
	const TArray<TVec3<FVec2f>>& FaceVertexPatternPositions,
	const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
	const FCollectionPropertyConstFacade& PropertyCollection)
	: Base(
		InParticles,
		InParticleOffset,
		InParticleCount,
		TriangleMesh.GetUniqueAdjacentElements(),
		TConstArrayView<FRealSingle>(), // We don't use base stiffness weight maps
		WeightMaps.FindRef(GetXPBDAnisoBucklingRatioString(PropertyCollection, XPBDAnisoBucklingRatioName.ToString())),
		TConstArrayView<FRealSingle>(), // We don't use base stiffness weight maps
		GetRestAngleMapFromCollection(WeightMaps, PropertyCollection),
		FSolverVec2(GetWeightedFloatXPBDAnisoBendingStiffnessWarp(PropertyCollection, MaxStiffness)),
		FSolverVec2(GetWeightedFloatXPBDAnisoBucklingRatio(PropertyCollection, 0.f)),
		FSolverVec2(GetWeightedFloatXPBDAnisoBucklingStiffnessWarp(PropertyCollection, MaxStiffness)),
		GetRestAngleValueFromCollection(PropertyCollection),
		(ERestAngleConstructionType)GetXPBDAnisoRestAngleType(PropertyCollection, (int32)ERestAngleConstructionType::Use3DRestAngles),
		true /*bTrimKinematicConstraints*/,
		MaxStiffness)
	, StiffnessWarp(
		FSolverVec2(GetWeightedFloatXPBDAnisoBendingStiffnessWarp(PropertyCollection, MaxStiffness)).ClampAxes(0, MaxStiffness),
		WeightMaps.FindRef(GetXPBDAnisoBendingStiffnessWarpString(PropertyCollection, XPBDAnisoBendingStiffnessWarpName.ToString())),
		TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
		ParticleOffset,
		ParticleCount)
	, StiffnessWeft(
		FSolverVec2(GetWeightedFloatXPBDAnisoBendingStiffnessWeft(PropertyCollection, MaxStiffness)).ClampAxes(0, MaxStiffness),
		WeightMaps.FindRef(GetXPBDAnisoBendingStiffnessWeftString(PropertyCollection, XPBDAnisoBendingStiffnessWeftName.ToString())),
		TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
		ParticleOffset,
		ParticleCount)
	, StiffnessBias(
		FSolverVec2(GetWeightedFloatXPBDAnisoBendingStiffnessBias(PropertyCollection, MaxStiffness)).ClampAxes(0, MaxStiffness),
		WeightMaps.FindRef(GetXPBDAnisoBendingStiffnessBiasString(PropertyCollection, XPBDAnisoBendingStiffnessBiasName.ToString())),
		TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
		ParticleOffset,
		ParticleCount)
	, BucklingStiffnessWarp(
		FSolverVec2(GetWeightedFloatXPBDAnisoBucklingStiffnessWarp(PropertyCollection, MaxStiffness)).ClampAxes(0, MaxStiffness),
		WeightMaps.FindRef(GetXPBDAnisoBucklingStiffnessWarpString(PropertyCollection, XPBDAnisoBucklingStiffnessWarpName.ToString())),
		TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
		ParticleOffset,
		ParticleCount)
	, BucklingStiffnessWeft(
		FSolverVec2(GetWeightedFloatXPBDAnisoBucklingStiffnessWeft(PropertyCollection, MaxStiffness)).ClampAxes(0, MaxStiffness),
		WeightMaps.FindRef(GetXPBDAnisoBucklingStiffnessWeftString(PropertyCollection, XPBDAnisoBucklingStiffnessWeftName.ToString())),
		TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
		ParticleOffset,
		ParticleCount)
	, BucklingStiffnessBias(
		FSolverVec2(GetWeightedFloatXPBDAnisoBucklingStiffnessBias(PropertyCollection, MaxStiffness)).ClampAxes(0, MaxStiffness),
		WeightMaps.FindRef(GetXPBDAnisoBucklingStiffnessBiasString(PropertyCollection, XPBDAnisoBucklingStiffnessBiasName.ToString())),
		TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
		ParticleOffset,
		ParticleCount)
	, DampingRatio(
		FSolverVec2(GetWeightedFloatXPBDAnisoBendingDamping(PropertyCollection, MinDamping)).ClampAxes(MinDamping, MaxDamping),
		WeightMaps.FindRef(GetXPBDAnisoBendingDampingString(PropertyCollection, XPBDAnisoBendingDampingName.ToString())),
		TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
		ParticleOffset,
		ParticleCount)
	, WarpWeftBiasBaseMultipliers(GenerateWarpWeftBiasBaseMultipliers(FaceVertexPatternPositions, TriangleMesh))
	, XPBDAnisoBendingStiffnessWarpIndex(PropertyCollection)
	, XPBDAnisoBendingStiffnessWeftIndex(PropertyCollection)
	, XPBDAnisoBendingStiffnessBiasIndex(PropertyCollection)
	, XPBDAnisoBendingDampingIndex(PropertyCollection)
	, XPBDAnisoBucklingRatioIndex(PropertyCollection)
	, XPBDAnisoBucklingStiffnessWarpIndex(PropertyCollection)
	, XPBDAnisoBucklingStiffnessWeftIndex(PropertyCollection)
	, XPBDAnisoBucklingStiffnessBiasIndex(PropertyCollection)
	, XPBDAnisoFlatnessRatioIndex(PropertyCollection)
	, XPBDAnisoRestAngleIndex(PropertyCollection)
	, XPBDAnisoRestAngleTypeIndex(PropertyCollection)
{
	Lambdas.Init((FSolverReal)0., Constraints.Num());
	LambdasDamping.Init((FSolverReal)0., Constraints.Num());
	InitColor(InParticles);
}

FXPBDAnisotropicBendingConstraints::FXPBDAnisotropicBendingConstraints(const FSolverParticles& InParticles,
	int32 ParticleOffset,
	int32 ParticleCount,
	const FTriangleMesh& TriangleMesh,
	const TArray<TVec3<FVec2f>>& FaceVertexPatternPositions,
	const TConstArrayView<FRealSingle>& StiffnessWarpMultipliers,
	const TConstArrayView<FRealSingle>& StiffnessWeftMultipliers,
	const TConstArrayView<FRealSingle>& StiffnessBiasMultipliers,
	const TConstArrayView<FRealSingle>& BucklingRatioMultipliers,
	const TConstArrayView<FRealSingle>& BucklingStiffnessWarpMultipliers,
	const TConstArrayView<FRealSingle>& BucklingStiffnessWeftMultipliers,
	const TConstArrayView<FRealSingle>& BucklingStiffnessBiasMultipliers,
	const TConstArrayView<FRealSingle>& DampingMultipliers,
	const TConstArrayView<FRealSingle>& RestAngleMap,
	const FSolverVec2& InStiffnessWarp,
	const FSolverVec2& InStiffnessWeft,
	const FSolverVec2& InStiffnessBias,
	const FSolverVec2& InBucklingRatio,
	const FSolverVec2& InBucklingStiffnessWarp,
	const FSolverVec2& InBucklingStiffnessWeft,
	const FSolverVec2& InBucklingStiffnessBias,
	const FSolverVec2& InDampingRatio,
	const FSolverVec2& RestAngleValue,
	ERestAngleConstructionType RestAngleConstructionType)
	: Base(
		InParticles,
		ParticleOffset,
		ParticleCount,
		TriangleMesh.GetUniqueAdjacentElements(),
		TConstArrayView<FRealSingle>(), // We don't use base stiffness weight maps
		BucklingRatioMultipliers,
		TConstArrayView<FRealSingle>(), // We don't use base stiffness weight maps
		RestAngleMap,
		InStiffnessWarp,
		InBucklingRatio,
		InBucklingStiffnessWarp,
		RestAngleValue,
		RestAngleConstructionType,
		true /*bTrimKinematicConstraints*/,
		MaxStiffness)
	, StiffnessWarp(
		InStiffnessWarp.ClampAxes(0, MaxStiffness),
		StiffnessWarpMultipliers,
		TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
		ParticleOffset,
		ParticleCount)
	, StiffnessWeft(
		InStiffnessWeft.ClampAxes(0, MaxStiffness),
		StiffnessWeftMultipliers,
		TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
		ParticleOffset,
		ParticleCount)
	, StiffnessBias(
		InStiffnessBias.ClampAxes(0, MaxStiffness),
		StiffnessBiasMultipliers,
		TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
		ParticleOffset,
		ParticleCount)
	, BucklingStiffnessWarp(
		InBucklingStiffnessWarp.ClampAxes(0, MaxStiffness),
		BucklingStiffnessWarpMultipliers,
		TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
		ParticleOffset,
		ParticleCount)
	, BucklingStiffnessWeft(
		InBucklingStiffnessWeft.ClampAxes(0, MaxStiffness),
		BucklingStiffnessWeftMultipliers,
		TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
		ParticleOffset,
		ParticleCount)
	, BucklingStiffnessBias(
		InBucklingStiffnessBias.ClampAxes(0, MaxStiffness),
		BucklingStiffnessBiasMultipliers,
		TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
		ParticleOffset,
		ParticleCount)
	, DampingRatio(
		InDampingRatio.ClampAxes((FSolverReal)0., (FSolverReal)1.),
		DampingMultipliers,
		TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
		ParticleOffset,
		ParticleCount)
	, WarpWeftBiasBaseMultipliers(GenerateWarpWeftBiasBaseMultipliers(FaceVertexPatternPositions, TriangleMesh))
	, XPBDAnisoBendingStiffnessWarpIndex(ForceInit)
	, XPBDAnisoBendingStiffnessWeftIndex(ForceInit)
	, XPBDAnisoBendingStiffnessBiasIndex(ForceInit)
	, XPBDAnisoBendingDampingIndex(ForceInit)
	, XPBDAnisoBucklingRatioIndex(ForceInit)
	, XPBDAnisoBucklingStiffnessWarpIndex(ForceInit)
	, XPBDAnisoBucklingStiffnessWeftIndex(ForceInit)
	, XPBDAnisoBucklingStiffnessBiasIndex(ForceInit)
	, XPBDAnisoFlatnessRatioIndex(ForceInit)
	, XPBDAnisoRestAngleIndex(ForceInit)
	, XPBDAnisoRestAngleTypeIndex(ForceInit)
{
	Lambdas.Init((FSolverReal)0., Constraints.Num());
	LambdasDamping.Init((FSolverReal)0., Constraints.Num());
	InitColor(InParticles);
}

TArray<FSolverVec3> FXPBDAnisotropicBendingConstraints::GenerateWarpWeftBiasBaseMultipliers(const TArray<TVec3<FVec2f>>& FaceVertexPatternPositions, const FTriangleMesh& TriangleMesh) const
{
	// This is easier to do by faces. Average the multipliers for faces that share the same edge.
	TMap<TVec2<int32> /*Edge*/, TArray<FSolverVec3> /*BaseMultipliers*/> EdgeToMultiplierMap;
	auto SortedEdge = [](int32 P0, int32 P1) { return P0 <= P1 ? TVec2<int32>(P0, P1) : TVec2<int32>(P1, P0); };
	auto Multiplier = [](const FVec2f& UV0, const FVec2f& UV1)
	{
		// Input UVs: X = Weft, Y = Warp direction
		// Transform to X = Weft, Y = Warp, Z = Bias
		// NOTE: Weft-dominant bend is along a vertical edge (i.e., larger V(Y) diff)). Warp-dominant bend is along a horizontal edge.
		const FSolverVec2 UVDiff = UV1 - UV0;
		const FSolverVec2 UVDiffAbs = UVDiff.GetAbs();
		FSolverVec3 UVDiffTransformed;
		if (UVDiffAbs.X > UVDiffAbs.Y)
		{
			// Stiffness is blend between warp and bias
			UVDiffTransformed  = FSolverVec3((FSolverReal)0.f, UVDiffAbs.X - UVDiffAbs.Y, UVDiffAbs.Y);
		}
		else
		{
			// Stiffness is blend between weft and bias
			UVDiffTransformed = FSolverVec3(UVDiffAbs.Y - UVDiffAbs.X, (FSolverReal)0.f, UVDiffAbs.X);
		}
		const FSolverReal Denom = UVDiffTransformed.X + UVDiffTransformed.Y + UVDiffTransformed.Z;
		return Denom > UE_SMALL_NUMBER ? UVDiffTransformed / Denom : FSolverVec3(0.f, 1.f, 0.f); // Default to Warp if zero length
	};

	const TArray<TVec3<int32>>& Elements = TriangleMesh.GetElements();
	for (int32 ElemIdx = 0; ElemIdx < Elements.Num(); ++ElemIdx)
	{
		const TVec3<int32>& Element = Elements[ElemIdx];
		const TVec3<FVec2f>& UVs = FaceVertexPatternPositions[ElemIdx];
		EdgeToMultiplierMap.FindOrAdd(SortedEdge(Element[0], Element[1])).Add(Multiplier(UVs[0], UVs[1]));
		EdgeToMultiplierMap.FindOrAdd(SortedEdge(Element[1], Element[2])).Add(Multiplier(UVs[1], UVs[2]));
		EdgeToMultiplierMap.FindOrAdd(SortedEdge(Element[2], Element[0])).Add(Multiplier(UVs[2], UVs[0]));
	}

	TArray<FSolverVec3> WarpWeftBaseMultiplierResult;
	WarpWeftBaseMultiplierResult.SetNumUninitialized(Constraints.Num());
	for (int32 ConstraintIdx = 0; ConstraintIdx < Constraints.Num(); ++ConstraintIdx)
	{
		const TArray<FSolverVec3>& EdgeMultipliers = EdgeToMultiplierMap.FindChecked(SortedEdge(ConstraintSharedEdges[ConstraintIdx][0], ConstraintSharedEdges[ConstraintIdx][1]));
		FSolverVec3 AveragedEdgeMultiplier((FSolverReal)0.);
		if (ensure(EdgeMultipliers.Num()))
		{
			for (const FSolverVec3& EdgeMult : EdgeMultipliers)
			{
				AveragedEdgeMultiplier += EdgeMult;
			}
			AveragedEdgeMultiplier /= (FSolverReal)EdgeMultipliers.Num();
		}
		WarpWeftBaseMultiplierResult[ConstraintIdx] = AveragedEdgeMultiplier;
	}
	return WarpWeftBaseMultiplierResult;
}

template<typename SolverParticlesOrRange>
void FXPBDAnisotropicBendingConstraints::InitColor(const SolverParticlesOrRange& InParticles)
{
	// In dev builds we always color so we can tune the system without restarting. See Apply()
#if UE_BUILD_SHIPPING || UE_BUILD_TEST
	if (Constraints.Num() > Chaos_XPBDBending_ParallelConstraintCount)
#endif
	{
		const TArray<TArray<int32>> ConstraintsPerColor = FGraphColoring::ComputeGraphColoringParticlesOrRange(Constraints, InParticles, ParticleOffset, ParticleOffset + ParticleCount);

		// Reorder constraints based on color so each array in ConstraintsPerColor contains contiguous elements.
		TArray<TVec4<int32>> ReorderedConstraints; 
		TArray<TVec2<int32>> ReorderedConstraintSharedEdges;
		TArray<FSolverReal> ReorderedRestAngles;
		TArray<FSolverVec3> ReorderedWarpWeftBiasBaseMultipliers;
		TArray<int32> OrigToReorderedIndices; // used to reorder stiffness indices
		ReorderedConstraints.SetNumUninitialized(Constraints.Num());
		ReorderedConstraintSharedEdges.SetNumUninitialized(Constraints.Num());
		ReorderedRestAngles.SetNumUninitialized(Constraints.Num());
		ReorderedWarpWeftBiasBaseMultipliers.SetNumUninitialized(Constraints.Num());
		OrigToReorderedIndices.SetNumUninitialized(Constraints.Num());

		ConstraintsPerColorStartIndex.Reset(ConstraintsPerColor.Num() + 1);

		int32 ReorderedIndex = 0;
		for (const TArray<int32>& ConstraintsBatch : ConstraintsPerColor)
		{
			ConstraintsPerColorStartIndex.Add(ReorderedIndex);
			for (const int32& BatchConstraint : ConstraintsBatch)
			{
				const int32 OrigIndex = BatchConstraint;
				ReorderedConstraints[ReorderedIndex] = Constraints[OrigIndex];
				ReorderedConstraintSharedEdges[ReorderedIndex] = ConstraintSharedEdges[OrigIndex];
				ReorderedRestAngles[ReorderedIndex] = RestAngles[OrigIndex];
				ReorderedWarpWeftBiasBaseMultipliers[ReorderedIndex] = WarpWeftBiasBaseMultipliers[OrigIndex];
				OrigToReorderedIndices[OrigIndex] = ReorderedIndex;
				++ReorderedIndex;
			}
		}
		ConstraintsPerColorStartIndex.Add(ReorderedIndex);

		Constraints = MoveTemp(ReorderedConstraints);
		ConstraintSharedEdges = MoveTemp(ReorderedConstraintSharedEdges);
		RestAngles = MoveTemp(ReorderedRestAngles);
		WarpWeftBiasBaseMultipliers = MoveTemp(ReorderedWarpWeftBiasBaseMultipliers);
		StiffnessWarp.ReorderIndices(OrigToReorderedIndices);
		StiffnessWeft.ReorderIndices(OrigToReorderedIndices);
		StiffnessBias.ReorderIndices(OrigToReorderedIndices);
		BucklingRatioWeighted.ReorderIndices(OrigToReorderedIndices);
		BucklingStiffnessWarp.ReorderIndices(OrigToReorderedIndices);
		BucklingStiffnessWeft.ReorderIndices(OrigToReorderedIndices);
		BucklingStiffnessBias.ReorderIndices(OrigToReorderedIndices);
		DampingRatio.ReorderIndices(OrigToReorderedIndices);

#if INTEL_ISPC
		ConstraintsIndex1.SetNumUninitialized(Constraints.Num());
		ConstraintsIndex2.SetNumUninitialized(Constraints.Num());
		ConstraintsIndex3.SetNumUninitialized(Constraints.Num());
		ConstraintsIndex4.SetNumUninitialized(Constraints.Num());
		for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
		{
			ConstraintsIndex1[ConstraintIndex] = Constraints[ConstraintIndex][0];
			ConstraintsIndex2[ConstraintIndex] = Constraints[ConstraintIndex][1];
			ConstraintsIndex3[ConstraintIndex] = Constraints[ConstraintIndex][2];
			ConstraintsIndex4[ConstraintIndex] = Constraints[ConstraintIndex][3];
		}
#endif
	}
}

void FXPBDAnisotropicBendingConstraints::SetProperties(
	const FCollectionPropertyConstFacade& PropertyCollection,
	const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps)
{
	if (IsXPBDAnisoBendingStiffnessWarpMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedValue(GetWeightedFloatXPBDAnisoBendingStiffnessWarp(PropertyCollection).ClampAxes(0, MaxStiffness));
		if (IsXPBDAnisoBendingStiffnessWarpStringDirty(PropertyCollection))
		{
			const FString& WeightMapName = GetXPBDAnisoBendingStiffnessWarpString(PropertyCollection);
			StiffnessWarp = FPBDFlatWeightMap(
				WeightedValue,
				WeightMaps.FindRef(WeightMapName),
				TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
				ParticleOffset,
				ParticleCount);
		}
		else
		{
			StiffnessWarp.SetWeightedValue(WeightedValue);
		}
	}
	if (IsXPBDAnisoBendingStiffnessWeftMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedValue(GetWeightedFloatXPBDAnisoBendingStiffnessWeft(PropertyCollection).ClampAxes(0, MaxStiffness));
		if (IsXPBDAnisoBendingStiffnessWeftStringDirty(PropertyCollection))
		{
			const FString& WeightMapName = GetXPBDAnisoBendingStiffnessWeftString(PropertyCollection);
			StiffnessWeft = FPBDFlatWeightMap(
				WeightedValue,
				WeightMaps.FindRef(WeightMapName),
				TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
				ParticleOffset,
				ParticleCount);
		}
		else
		{
			StiffnessWeft.SetWeightedValue(WeightedValue);
		}
	}
	if (IsXPBDAnisoBendingStiffnessBiasMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedValue(GetWeightedFloatXPBDAnisoBendingStiffnessBias(PropertyCollection).ClampAxes(0, MaxStiffness));
		if (IsXPBDAnisoBendingStiffnessBiasStringDirty(PropertyCollection))
		{
			const FString& WeightMapName = GetXPBDAnisoBendingStiffnessBiasString(PropertyCollection);
			StiffnessBias = FPBDFlatWeightMap(
				WeightedValue,
				WeightMaps.FindRef(WeightMapName),
				TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
				ParticleOffset,
				ParticleCount);
		}
		else
		{
			StiffnessBias.SetWeightedValue(WeightedValue);
		}
	}
	if (IsXPBDAnisoBucklingRatioMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedValue = FSolverVec2(GetWeightedFloatXPBDAnisoBucklingRatio(PropertyCollection)).ClampAxes(0.f, 1.f);
		if (IsXPBDAnisoBucklingRatioStringDirty(PropertyCollection))
		{
			const FString& WeightMapName = GetXPBDAnisoBucklingRatioString(PropertyCollection);
			BucklingRatioWeighted = FPBDWeightMap(
				WeightedValue,
				WeightMaps.FindRef(WeightMapName),
				TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
				ParticleOffset,
				ParticleCount);
		}
		else
		{
			BucklingRatioWeighted.SetWeightedValue(WeightedValue);
		}
	}
	if (IsXPBDAnisoBucklingStiffnessWarpMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedValue(GetWeightedFloatXPBDAnisoBucklingStiffnessWarp(PropertyCollection).ClampAxes(0, MaxStiffness));
		if (IsXPBDAnisoBucklingStiffnessWarpStringDirty(PropertyCollection))
		{
			const FString& WeightMapName = GetXPBDAnisoBucklingStiffnessWarpString(PropertyCollection);
			BucklingStiffnessWarp = FPBDFlatWeightMap(
				WeightedValue,
				WeightMaps.FindRef(WeightMapName),
				TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
				ParticleOffset,
				ParticleCount);
		}
		else
		{
			BucklingStiffnessWarp.SetWeightedValue(WeightedValue);
		}
	}
	if (IsXPBDAnisoBucklingStiffnessWeftMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedValue(GetWeightedFloatXPBDAnisoBucklingStiffnessWeft(PropertyCollection).ClampAxes(0, MaxStiffness));
		if (IsXPBDAnisoBucklingStiffnessWeftStringDirty(PropertyCollection))
		{
			const FString& WeightMapName = GetXPBDAnisoBucklingStiffnessWeftString(PropertyCollection);
			BucklingStiffnessWeft = FPBDFlatWeightMap(
				WeightedValue,
				WeightMaps.FindRef(WeightMapName),
				TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
				ParticleOffset,
				ParticleCount);
		}
		else
		{
			BucklingStiffnessWeft.SetWeightedValue(WeightedValue);
		}
	}
	if (IsXPBDAnisoBucklingStiffnessBiasMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedValue(GetWeightedFloatXPBDAnisoBucklingStiffnessBias(PropertyCollection).ClampAxes(0, MaxStiffness));
		if (IsXPBDAnisoBucklingStiffnessBiasStringDirty(PropertyCollection))
		{
			const FString& WeightMapName = GetXPBDAnisoBucklingStiffnessBiasString(PropertyCollection);
			BucklingStiffnessBias = FPBDFlatWeightMap(
				WeightedValue,
				WeightMaps.FindRef(WeightMapName),
				TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
				ParticleOffset,
				ParticleCount);
		}
		else
		{
			BucklingStiffnessBias.SetWeightedValue(WeightedValue);
		}
	}
	if (IsXPBDAnisoBendingDampingMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedValue = FSolverVec2(GetXPBDAnisoBendingDamping(PropertyCollection)).ClampAxes(MinDamping, MaxDamping);
		if (IsXPBDAnisoBendingDampingStringDirty(PropertyCollection))
		{
			const FString& WeightMapName = GetXPBDAnisoBendingDampingString(PropertyCollection);
			DampingRatio = FPBDFlatWeightMap(
				WeightedValue,
				WeightMaps.FindRef(WeightMapName),
				TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
				ParticleOffset,
				ParticleCount);
		}
		else
		{
			DampingRatio.SetWeightedValue(WeightedValue);
		}
	}
}

template<bool bDampingOnly, bool bElasticOnly, typename SolverParticlesOrRange>
void FXPBDAnisotropicBendingConstraints::ApplyHelper(SolverParticlesOrRange& Particles, const FSolverReal Dt, const int32 ConstraintIndex, const FSolverVec3& ExpStiffnessValues,
	const FSolverVec3& ExpBucklingStiffnessValues, const FSolverReal DampingRatioValue) const
{
	const TVec4<int32>& Constraint = Constraints.GetData()[ConstraintIndex];
	const int32 Index1 = Constraint[0];
	const int32 Index2 = Constraint[1];
	const int32 Index3 = Constraint[2];
	const int32 Index4 = Constraint[3];
	const FSolverVec3& WarpWeftBiasBaseMultiplier = WarpWeftBiasBaseMultipliers.GetData()[ConstraintIndex];

	const FSolverReal BiphasicStiffnessValue = IsBuckled.GetData()[ConstraintIndex] ?
		WarpWeftBiasBaseMultiplier.Dot(ExpBucklingStiffnessValues): WarpWeftBiasBaseMultiplier.Dot(ExpStiffnessValues);

	const FSolverReal InvM1 = Particles.GetPAndInvM().GetData()[Index1].InvM;
	const FSolverReal InvM2 = Particles.GetPAndInvM().GetData()[Index2].InvM;
	const FSolverReal InvM3 = Particles.GetPAndInvM().GetData()[Index3].InvM;
	const FSolverReal InvM4 = Particles.GetPAndInvM().GetData()[Index4].InvM;

	if (BiphasicStiffnessValue <= MinStiffness)
	{
		return;
	}
	const FSolverReal CombinedInvMass = InvM1 + InvM2 + InvM3 + InvM4;
	check(CombinedInvMass > 0);

	FSolverReal ElasticTerm(0.f), DampingTerm(0.f), DenomScale(0.f);
	const FSolverVec3 P1 = Particles.GetPAndInvM().GetData()[Index1].P;
	const FSolverVec3 P2 = Particles.GetPAndInvM().GetData()[Index2].P;
	const FSolverVec3 P3 = Particles.GetPAndInvM().GetData()[Index3].P;
	const FSolverVec3 P4 = Particles.GetPAndInvM().GetData()[Index4].P;

	FSolverReal Angle(0.f);
	const TStaticArray<FSolverVec3, 4> Grads = Base::CalcGradients(P1, P2, P3, P4, bDampingOnly ? nullptr : &Angle);

	if constexpr (!bDampingOnly)
	{
		const FSolverReal AlphaInv = BiphasicStiffnessValue * Dt * Dt;
		ElasticTerm = AlphaInv * (Angle - RestAngles.GetData()[ConstraintIndex]);

		DenomScale += AlphaInv;
	}
	if constexpr (!bElasticOnly)
	{
		const FSolverReal Damping = (FSolverReal)2.f * DampingRatioValue * FMath::Sqrt(BiphasicStiffnessValue / CombinedInvMass);
		const FSolverReal BetaDt = Damping * Dt;

		const FSolverVec3 V1TimesDt = P1 - Particles.XArray().GetData()[Index1];
		const FSolverVec3 V2TimesDt = P2 - Particles.XArray().GetData()[Index2];
		const FSolverVec3 V3TimesDt = P3 - Particles.XArray().GetData()[Index3];
		const FSolverVec3 V4TimesDt = P4 - Particles.XArray().GetData()[Index4];

		DampingTerm = BetaDt * (FSolverVec3::DotProduct(V1TimesDt, Grads[0]) + FSolverVec3::DotProduct(V2TimesDt, Grads[1]) + FSolverVec3::DotProduct(V3TimesDt, Grads[2]) + FSolverVec3::DotProduct(V4TimesDt, Grads[3]));

		DenomScale += BetaDt;
	}

	FSolverReal& Lambda = bDampingOnly ? LambdasDamping.GetData()[ConstraintIndex] : Lambdas.GetData()[ConstraintIndex];

	const FSolverReal Denom = (DenomScale) * (InvM1 * Grads[0].SizeSquared() + InvM2 * Grads[1].SizeSquared() + InvM3 * Grads[2].SizeSquared() + InvM4 * Grads[3].SizeSquared()) + (FSolverReal)1.;
	const FSolverReal DLambda = (-ElasticTerm - DampingTerm - Lambda) / Denom;

	Particles.GetPAndInvM().GetData()[Index1].P += DLambda * InvM1 * Grads[0];
	Particles.GetPAndInvM().GetData()[Index2].P += DLambda * InvM2 * Grads[1];
	Particles.GetPAndInvM().GetData()[Index3].P += DLambda * InvM3 * Grads[2];
	Particles.GetPAndInvM().GetData()[Index4].P += DLambda * InvM4 * Grads[3];
	Lambda += DLambda;
}

template<typename SolverParticlesOrRange>
void FXPBDAnisotropicBendingConstraints::Init(const SolverParticlesOrRange& Particles)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FXPBDAnisotropicBendingConstraints_Init);
	Lambdas.Reset();
	Lambdas.AddZeroed(Constraints.Num());
	LambdasDamping.Reset();
	LambdasDamping.AddZeroed(Constraints.Num());
#if INTEL_ISPC
	IsBuckled.SetNumUninitialized(Constraints.Num());
	X1Array.SetNumUninitialized(Constraints.Num());
	X2Array.SetNumUninitialized(Constraints.Num());
	X3Array.SetNumUninitialized(Constraints.Num());
	X4Array.SetNumUninitialized(Constraints.Num());
	if (bRealTypeCompatibleWithISPC && bChaos_Bending_ISPC_Enabled && ConstraintsIndex1.Num() == Constraints.Num())
	{
		if (BucklingRatioWeighted.HasWeightMap())
		{
			ispc::InitXPBDBendingConstraintsIsBuckledWithMaps(
				(const ispc::FVector3f*)Particles.XArray().GetData(),
				ConstraintsIndex1.GetData(),
				ConstraintsIndex2.GetData(),
				ConstraintsIndex3.GetData(),
				ConstraintsIndex4.GetData(),
				RestAngles.GetData(),
				IsBuckled.GetData(),
				(ispc::FVector3f*)X1Array.GetData(),
				(ispc::FVector3f*)X2Array.GetData(),
				(ispc::FVector3f*)X3Array.GetData(),
				(ispc::FVector3f*)X4Array.GetData(),
				BucklingRatioWeighted.GetIndices().GetData(),
				BucklingRatioWeighted.GetTable().GetData(),
				Constraints.Num()
			);
		}
		else
		{
			ispc::InitXPBDBendingConstraintsIsBuckled(
				(const ispc::FVector3f*)Particles.XArray().GetData(),
				ConstraintsIndex1.GetData(),
				ConstraintsIndex2.GetData(),
				ConstraintsIndex3.GetData(),
				ConstraintsIndex4.GetData(),
				RestAngles.GetData(),
				IsBuckled.GetData(),
				(ispc::FVector3f*)X1Array.GetData(),
				(ispc::FVector3f*)X2Array.GetData(),
				(ispc::FVector3f*)X3Array.GetData(),
				(ispc::FVector3f*)X4Array.GetData(),
				(FSolverReal)BucklingRatioWeighted,
				Constraints.Num()
			);
		}
	}
	else
#endif
	{
		FPBDBendingConstraintsBase::Init(Particles);
	}
}
template CHAOS_API void FXPBDAnisotropicBendingConstraints::Init(const FSolverParticles& Particles);
template CHAOS_API void FXPBDAnisotropicBendingConstraints::Init(const FSolverParticlesRange& Particles);

template<typename SolverParticlesOrRange>
void FXPBDAnisotropicBendingConstraints::Apply(SolverParticlesOrRange& Particles, const FSolverReal Dt) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FXPBDAnisotropicBendingConstraints_Apply);
	SCOPE_CYCLE_COUNTER(STAT_XPBD_AnisoBending);

	const bool StiffnessHasWeightMap = StiffnessWarp.HasWeightMap();
	const bool StiffnessWeftHasWeightMap = StiffnessWeft.HasWeightMap();
	const bool StiffnessBiasHasWeightMap = StiffnessBias.HasWeightMap();
	const bool BucklingStiffnessHasWeightMap = BucklingStiffnessWarp.HasWeightMap();
	const bool BucklingStiffnessWeftHasWeightMap = BucklingStiffnessWeft.HasWeightMap();
	const bool BucklingStiffnessBiasHasWeightMap = BucklingStiffnessBias.HasWeightMap();
	const bool DampingHasWeightMap = DampingRatio.HasWeightMap();

	if (ConstraintsPerColorStartIndex.Num() > 0 && Constraints.Num() > Chaos_XPBDBending_ParallelConstraintCount)
	{
		const int32 ConstraintColorNum = ConstraintsPerColorStartIndex.Num() - 1;
#if INTEL_ISPC
		if (bRealTypeCompatibleWithISPC && bChaos_XPBDBending_ISPC_Enabled)
		{
			if (!StiffnessHasWeightMap && !StiffnessWeftHasWeightMap && !StiffnessBiasHasWeightMap &&
				!BucklingStiffnessHasWeightMap && !BucklingStiffnessWeftHasWeightMap && !BucklingStiffnessBiasHasWeightMap &&
				!DampingHasWeightMap)
			{
				const FSolverVec3 ExpStiffnessValue((FSolverReal)StiffnessWeft, (FSolverReal)StiffnessWarp, (FSolverReal)StiffnessBias);
				const FSolverVec3 ExpBucklingValue((FSolverReal)BucklingStiffnessWeft, (FSolverReal)BucklingStiffnessWarp, (FSolverReal)BucklingStiffnessBias);
				const FSolverReal DampingRatioValue = (FSolverReal)DampingRatio;

				if (ExpStiffnessValue.Max() <= MinStiffness && ExpBucklingValue.Max() <= MinStiffness)
				{
					return;
				}

				if (DampingRatioValue > 0)
				{
					if (bChaos_XPBDBending_SplitLambdaDamping)
					{
						for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
						{
							const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
							const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;	
							if (ColorSize >= Chaos_XPBDBending_ISPC_ParallelBatchSize * Chaos_XPBDBending_ISPC_MinNumParallelBatches)
							{
								const int32 NumBatches = FMath::DivideAndRoundUp(ColorSize, Chaos_XPBDBending_ISPC_ParallelBatchSize);
								PhysicsParallelFor(NumBatches, [this, &Particles, ColorStart, ColorSize,
									ParallelBatchSize = Chaos_XPBDBending_ISPC_ParallelBatchSize,
									Dt, &ExpStiffnessValue, &ExpBucklingValue, DampingRatioValue](const int32 BatchIndex)
								{
									const int32 BatchStart = BatchIndex * ParallelBatchSize;
									const int32 BatchEnd = FMath::Min((BatchIndex + 1) * ParallelBatchSize, ColorSize);
									ispc::ApplyXPBDAnisotropicBendingDampingConstraints(
										(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
										(const ispc::FVector3f*)&X1Array.GetData()[ColorStart + BatchStart],
										(const ispc::FVector3f*)&X2Array.GetData()[ColorStart + BatchStart],
										(const ispc::FVector3f*)&X3Array.GetData()[ColorStart + BatchStart],
										(const ispc::FVector3f*)&X4Array.GetData()[ColorStart + BatchStart],
										&ConstraintsIndex1.GetData()[ColorStart + BatchStart],
										&ConstraintsIndex2.GetData()[ColorStart + BatchStart],
										&ConstraintsIndex3.GetData()[ColorStart + BatchStart],
										&ConstraintsIndex4.GetData()[ColorStart + BatchStart],
										&RestAngles.GetData()[ColorStart + BatchStart],
										&IsBuckled.GetData()[ColorStart + BatchStart],
										(const ispc::FVector3f*)&WarpWeftBiasBaseMultipliers.GetData()[ColorStart + BatchStart],
										&LambdasDamping.GetData()[ColorStart + BatchStart],
										Dt,
										reinterpret_cast<const ispc::FVector3f&>(ExpStiffnessValue),
										reinterpret_cast<const ispc::FVector3f&>(ExpBucklingValue),
										DampingRatioValue,
										BatchEnd - BatchStart);
								});
							}
							else
							{
								ispc::ApplyXPBDAnisotropicBendingDampingConstraints(
									(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
									(const ispc::FVector3f*)&X1Array.GetData()[ColorStart],
									(const ispc::FVector3f*)&X2Array.GetData()[ColorStart],
									(const ispc::FVector3f*)&X3Array.GetData()[ColorStart],
									(const ispc::FVector3f*)&X4Array.GetData()[ColorStart],
									&ConstraintsIndex1.GetData()[ColorStart],
									&ConstraintsIndex2.GetData()[ColorStart],
									&ConstraintsIndex3.GetData()[ColorStart],
									&ConstraintsIndex4.GetData()[ColorStart],
									&RestAngles.GetData()[ColorStart],
									&IsBuckled.GetData()[ColorStart],
									(const ispc::FVector3f*)&WarpWeftBiasBaseMultipliers.GetData()[ColorStart],
									&LambdasDamping.GetData()[ColorStart],
									Dt,
									reinterpret_cast<const ispc::FVector3f&>(ExpStiffnessValue),
									reinterpret_cast<const ispc::FVector3f&>(ExpBucklingValue),
									DampingRatioValue,
									ColorSize);
							}
						}
					}
					else
					{
						for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
						{
							const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
							const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
							if (ColorSize >= Chaos_XPBDBending_ISPC_ParallelBatchSize * Chaos_XPBDBending_ISPC_MinNumParallelBatches)
							{
								const int32 NumBatches = FMath::DivideAndRoundUp(ColorSize, Chaos_XPBDBending_ISPC_ParallelBatchSize);
								PhysicsParallelFor(NumBatches, [this, &Particles, ColorStart, ColorSize,
									ParallelBatchSize = Chaos_XPBDBending_ISPC_ParallelBatchSize,
									Dt, &ExpStiffnessValue, &ExpBucklingValue, DampingRatioValue](const int32 BatchIndex)
								{
									const int32 BatchStart = BatchIndex * ParallelBatchSize;
									const int32 BatchEnd = FMath::Min((BatchIndex + 1) * ParallelBatchSize, ColorSize);
									ispc::ApplyXPBDAnisotropicBendingConstraintsWithDamping(
										(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
										(const ispc::FVector3f*)&X1Array.GetData()[ColorStart + BatchStart],
										(const ispc::FVector3f*)&X2Array.GetData()[ColorStart + BatchStart],
										(const ispc::FVector3f*)&X3Array.GetData()[ColorStart + BatchStart],
										(const ispc::FVector3f*)&X4Array.GetData()[ColorStart + BatchStart],
										&ConstraintsIndex1.GetData()[ColorStart + BatchStart],
										&ConstraintsIndex2.GetData()[ColorStart + BatchStart],
										&ConstraintsIndex3.GetData()[ColorStart + BatchStart],
										&ConstraintsIndex4.GetData()[ColorStart + BatchStart],
										&RestAngles.GetData()[ColorStart + BatchStart],
										&IsBuckled.GetData()[ColorStart + BatchStart],
										(const ispc::FVector3f*)&WarpWeftBiasBaseMultipliers.GetData()[ColorStart + BatchStart],
										&Lambdas.GetData()[ColorStart + BatchStart],
										Dt,
										reinterpret_cast<const ispc::FVector3f&>(ExpStiffnessValue),
										reinterpret_cast<const ispc::FVector3f&>(ExpBucklingValue),
										DampingRatioValue,
										BatchEnd - BatchStart);
								});
							}
							else
							{
								ispc::ApplyXPBDAnisotropicBendingConstraintsWithDamping(
									(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
									(const ispc::FVector3f*)&X1Array.GetData()[ColorStart],
									(const ispc::FVector3f*)&X2Array.GetData()[ColorStart],
									(const ispc::FVector3f*)&X3Array.GetData()[ColorStart],
									(const ispc::FVector3f*)&X4Array.GetData()[ColorStart],
									&ConstraintsIndex1.GetData()[ColorStart],
									&ConstraintsIndex2.GetData()[ColorStart],
									&ConstraintsIndex3.GetData()[ColorStart],
									&ConstraintsIndex4.GetData()[ColorStart],
									&RestAngles.GetData()[ColorStart],
									&IsBuckled.GetData()[ColorStart],
									(const ispc::FVector3f*)&WarpWeftBiasBaseMultipliers.GetData()[ColorStart],
									&Lambdas.GetData()[ColorStart],
									Dt,
									reinterpret_cast<const ispc::FVector3f&>(ExpStiffnessValue),
									reinterpret_cast<const ispc::FVector3f&>(ExpBucklingValue),
									DampingRatioValue,
									ColorSize);
							}
						}
						return;
					}
				}
				for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
				{
					const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
					const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
					if (ColorSize >= Chaos_XPBDBending_ISPC_ParallelBatchSize * Chaos_XPBDBending_ISPC_MinNumParallelBatches)
					{
						const int32 NumBatches = FMath::DivideAndRoundUp(ColorSize, Chaos_XPBDBending_ISPC_ParallelBatchSize);
						PhysicsParallelFor(NumBatches, [this, &Particles, ColorStart, ColorSize,
							ParallelBatchSize = Chaos_XPBDBending_ISPC_ParallelBatchSize,
							Dt, &ExpStiffnessValue, &ExpBucklingValue](const int32 BatchIndex)
						{
							const int32 BatchStart = BatchIndex * ParallelBatchSize;
							const int32 BatchEnd = FMath::Min((BatchIndex + 1) * ParallelBatchSize, ColorSize);
							ispc::ApplyXPBDAnisotropicBendingConstraints(
								(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
								&ConstraintsIndex1.GetData()[ColorStart + BatchStart],
								&ConstraintsIndex2.GetData()[ColorStart + BatchStart],
								&ConstraintsIndex3.GetData()[ColorStart + BatchStart],
								&ConstraintsIndex4.GetData()[ColorStart + BatchStart],
								&RestAngles.GetData()[ColorStart + BatchStart],
								&IsBuckled.GetData()[ColorStart + BatchStart],
								(const ispc::FVector3f*)&WarpWeftBiasBaseMultipliers.GetData()[ColorStart + BatchStart],
								&Lambdas.GetData()[ColorStart + BatchStart],
								Dt,
								reinterpret_cast<const ispc::FVector3f&>(ExpStiffnessValue),
								reinterpret_cast<const ispc::FVector3f&>(ExpBucklingValue),
								BatchEnd - BatchStart);
						});
					}
					else
					{
						ispc::ApplyXPBDAnisotropicBendingConstraints(
							(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
							&ConstraintsIndex1.GetData()[ColorStart],
							&ConstraintsIndex2.GetData()[ColorStart],
							&ConstraintsIndex3.GetData()[ColorStart],
							&ConstraintsIndex4.GetData()[ColorStart],
							&RestAngles.GetData()[ColorStart],
							&IsBuckled.GetData()[ColorStart],
							(const ispc::FVector3f*)&WarpWeftBiasBaseMultipliers.GetData()[ColorStart],
							&Lambdas.GetData()[ColorStart],
							Dt,
							reinterpret_cast<const ispc::FVector3f&>(ExpStiffnessValue),
							reinterpret_cast<const ispc::FVector3f&>(ExpBucklingValue),
							ColorSize);

					}
				}
			}
			else
			{
				// ISPC with maps
				if (DampingHasWeightMap || (FSolverReal)DampingRatio > 0)
				{
					if (bChaos_XPBDBending_SplitLambdaDamping)
					{
						for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
						{
							const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
							const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
							ispc::ApplyXPBDAnisotropicBendingDampingConstraintsWithMaps(
								(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
								(const ispc::FVector3f*)&X1Array.GetData()[ColorStart],
								(const ispc::FVector3f*)&X2Array.GetData()[ColorStart],
								(const ispc::FVector3f*)&X3Array.GetData()[ColorStart],
								(const ispc::FVector3f*)&X4Array.GetData()[ColorStart],
								&ConstraintsIndex1.GetData()[ColorStart],
								&ConstraintsIndex2.GetData()[ColorStart],
								&ConstraintsIndex3.GetData()[ColorStart],
								&ConstraintsIndex4.GetData()[ColorStart],
								&RestAngles.GetData()[ColorStart],
								&IsBuckled.GetData()[ColorStart],
								(const ispc::FVector3f*)&WarpWeftBiasBaseMultipliers.GetData()[ColorStart],
								&LambdasDamping.GetData()[ColorStart],
								Dt,
								StiffnessHasWeightMap,
								reinterpret_cast<const ispc::FVector2f&>(StiffnessWarp.GetOffsetRange()),
								StiffnessHasWeightMap ? &StiffnessWarp.GetMapValues().GetData()[ColorStart] : nullptr,
								StiffnessWeftHasWeightMap,
								reinterpret_cast<const ispc::FVector2f&>(StiffnessWeft.GetOffsetRange()),
								StiffnessWeftHasWeightMap ? &StiffnessWeft.GetMapValues().GetData()[ColorStart] : nullptr,
								StiffnessBiasHasWeightMap,
								reinterpret_cast<const ispc::FVector2f&>(StiffnessBias.GetOffsetRange()),
								StiffnessBiasHasWeightMap ? &StiffnessBias.GetMapValues().GetData()[ColorStart] : nullptr,
								BucklingStiffnessHasWeightMap,
								reinterpret_cast<const ispc::FVector2f&>(BucklingStiffnessWarp.GetOffsetRange()),
								BucklingStiffnessHasWeightMap ? &BucklingStiffnessWarp.GetMapValues().GetData()[ColorStart] : nullptr,
								BucklingStiffnessWeftHasWeightMap,
								reinterpret_cast<const ispc::FVector2f&>(BucklingStiffnessWeft.GetOffsetRange()),
								BucklingStiffnessWeftHasWeightMap ? &BucklingStiffnessWeft.GetMapValues().GetData()[ColorStart] : nullptr,
								BucklingStiffnessBiasHasWeightMap,
								reinterpret_cast<const ispc::FVector2f&>(BucklingStiffnessBias.GetOffsetRange()),
								BucklingStiffnessBiasHasWeightMap ? &BucklingStiffnessBias.GetMapValues().GetData()[ColorStart] : nullptr,
								DampingHasWeightMap,
								reinterpret_cast<const ispc::FVector2f&>(DampingRatio.GetOffsetRange()),
								DampingHasWeightMap ? &DampingRatio.GetMapValues()[ColorStart] : nullptr,
								ColorSize);
						}
					}
					else
					{
						for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
						{
							const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
							const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
							ispc::ApplyXPBDAnisotropicBendingConstraintsWithDampingAndMaps(
								(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
								(const ispc::FVector3f*)&X1Array.GetData()[ColorStart],
								(const ispc::FVector3f*)&X2Array.GetData()[ColorStart],
								(const ispc::FVector3f*)&X3Array.GetData()[ColorStart],
								(const ispc::FVector3f*)&X4Array.GetData()[ColorStart],
								&ConstraintsIndex1.GetData()[ColorStart],
								&ConstraintsIndex2.GetData()[ColorStart],
								&ConstraintsIndex3.GetData()[ColorStart],
								&ConstraintsIndex4.GetData()[ColorStart],
								&RestAngles.GetData()[ColorStart],
								&IsBuckled.GetData()[ColorStart],
								(const ispc::FVector3f*)&WarpWeftBiasBaseMultipliers.GetData()[ColorStart],
								&Lambdas.GetData()[ColorStart],
								Dt,
								StiffnessHasWeightMap,
								reinterpret_cast<const ispc::FVector2f&>(StiffnessWarp.GetOffsetRange()),
								StiffnessHasWeightMap ? &StiffnessWarp.GetMapValues().GetData()[ColorStart] : nullptr,
								StiffnessWeftHasWeightMap,
								reinterpret_cast<const ispc::FVector2f&>(StiffnessWeft.GetOffsetRange()),
								StiffnessWeftHasWeightMap ? &StiffnessWeft.GetMapValues().GetData()[ColorStart] : nullptr,
								StiffnessBiasHasWeightMap,
								reinterpret_cast<const ispc::FVector2f&>(StiffnessBias.GetOffsetRange()),
								StiffnessBiasHasWeightMap ? &StiffnessBias.GetMapValues().GetData()[ColorStart] : nullptr,
								BucklingStiffnessHasWeightMap,
								reinterpret_cast<const ispc::FVector2f&>(BucklingStiffnessWarp.GetOffsetRange()),
								BucklingStiffnessHasWeightMap ? &BucklingStiffnessWarp.GetMapValues().GetData()[ColorStart] : nullptr,
								BucklingStiffnessWeftHasWeightMap,
								reinterpret_cast<const ispc::FVector2f&>(BucklingStiffnessWeft.GetOffsetRange()),
								BucklingStiffnessWeftHasWeightMap ? &BucklingStiffnessWeft.GetMapValues().GetData()[ColorStart] : nullptr,
								BucklingStiffnessBiasHasWeightMap,
								reinterpret_cast<const ispc::FVector2f&>(BucklingStiffnessBias.GetOffsetRange()),
								BucklingStiffnessBiasHasWeightMap ? &BucklingStiffnessBias.GetMapValues().GetData()[ColorStart] : nullptr,
								DampingHasWeightMap,
								reinterpret_cast<const ispc::FVector2f&>(DampingRatio.GetOffsetRange()),
								DampingHasWeightMap ? &DampingRatio.GetMapValues()[ColorStart] : nullptr,
								ColorSize);
						}
						return;
					}
				}
				for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
				{
					const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
					const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
					ispc::ApplyXPBDAnisotropicBendingConstraintsWithMaps(
						(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
						&ConstraintsIndex1.GetData()[ColorStart],
						&ConstraintsIndex2.GetData()[ColorStart],
						&ConstraintsIndex3.GetData()[ColorStart],
						&ConstraintsIndex4.GetData()[ColorStart],
						&RestAngles.GetData()[ColorStart],
						&IsBuckled.GetData()[ColorStart],
						(const ispc::FVector3f*)&WarpWeftBiasBaseMultipliers.GetData()[ColorStart],
						&Lambdas.GetData()[ColorStart],
						Dt,
						StiffnessHasWeightMap,
						reinterpret_cast<const ispc::FVector2f&>(StiffnessWarp.GetOffsetRange()),
						StiffnessHasWeightMap ? &StiffnessWarp.GetMapValues().GetData()[ColorStart] : nullptr,
						StiffnessWeftHasWeightMap,
						reinterpret_cast<const ispc::FVector2f&>(StiffnessWeft.GetOffsetRange()),
						StiffnessWeftHasWeightMap ? &StiffnessWeft.GetMapValues().GetData()[ColorStart] : nullptr,
						StiffnessBiasHasWeightMap,
						reinterpret_cast<const ispc::FVector2f&>(StiffnessBias.GetOffsetRange()),
						StiffnessBiasHasWeightMap ? &StiffnessBias.GetMapValues().GetData()[ColorStart] : nullptr,
						BucklingStiffnessHasWeightMap,
						reinterpret_cast<const ispc::FVector2f&>(BucklingStiffnessWarp.GetOffsetRange()),
						BucklingStiffnessHasWeightMap ? &BucklingStiffnessWarp.GetMapValues().GetData()[ColorStart] : nullptr,
						BucklingStiffnessWeftHasWeightMap,
						reinterpret_cast<const ispc::FVector2f&>(BucklingStiffnessWeft.GetOffsetRange()),
						BucklingStiffnessWeftHasWeightMap ? &BucklingStiffnessWeft.GetMapValues().GetData()[ColorStart] : nullptr,
						BucklingStiffnessBiasHasWeightMap,
						reinterpret_cast<const ispc::FVector2f&>(BucklingStiffnessBias.GetOffsetRange()),
						BucklingStiffnessBiasHasWeightMap ? &BucklingStiffnessBias.GetMapValues().GetData()[ColorStart] : nullptr,
						ColorSize);
				}
			}
		}
		else
#endif
		{
			// Parallel non-ispc
			const FSolverReal StiffnessNoMap = (FSolverReal)StiffnessWarp;
			const FSolverReal StiffnessWeftNoMap = (FSolverReal)StiffnessWeft;
			const FSolverReal StiffnessBiasNoMap = (FSolverReal)StiffnessBias;
			const FSolverReal BucklingStiffnessNoMap = (FSolverReal)BucklingStiffnessWarp;
			const FSolverReal BucklingStiffnessWeftNoMap = (FSolverReal)BucklingStiffnessWeft;
			const FSolverReal BucklingStiffnessBiasNoMap = (FSolverReal)BucklingStiffnessBias;
			const FSolverReal DampingNoMap = (FSolverReal)DampingRatio;

			if (DampingHasWeightMap || (FSolverReal)DampingRatio > 0)
			{
				if (bChaos_XPBDBending_SplitLambdaDamping)
				{
					for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
					{
						const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
						const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
						PhysicsParallelFor(ColorSize, [&](const int32 Index)
						{
							const int32 ConstraintIndex = ColorStart + Index;
							const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? StiffnessWarp[ConstraintIndex] : StiffnessNoMap;
							const FSolverReal ExpStiffnessWeftValue = StiffnessWeftHasWeightMap ? StiffnessWeft[ConstraintIndex] : StiffnessWeftNoMap;
							const FSolverReal ExpStiffnessBiasValue = StiffnessBiasHasWeightMap ? StiffnessBias[ConstraintIndex] : StiffnessBiasNoMap;
							const FSolverReal ExpBucklingValue = BucklingStiffnessHasWeightMap ? BucklingStiffnessWarp[ConstraintIndex] : BucklingStiffnessNoMap;
							const FSolverReal ExpBucklingWeftValue = BucklingStiffnessWeftHasWeightMap ? BucklingStiffnessWeft[ConstraintIndex] : BucklingStiffnessWeftNoMap;
							const FSolverReal ExpBucklingBiasValue = BucklingStiffnessBiasHasWeightMap ? BucklingStiffnessBias[ConstraintIndex] : BucklingStiffnessBiasNoMap;
							const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
							constexpr bool bDampingOnly = true;
							constexpr bool bElasticOnly = false;
							ApplyHelper<bDampingOnly, bElasticOnly>(Particles, Dt, ConstraintIndex,
								FSolverVec3(ExpStiffnessWeftValue, ExpStiffnessValue, ExpStiffnessBiasValue), FSolverVec3(ExpBucklingWeftValue, ExpBucklingValue, ExpBucklingBiasValue), DampingRatioValue);
						});
					}
				}
				else
				{
					for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
					{
						const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
						const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
						PhysicsParallelFor(ColorSize, [&](const int32 Index)
						{
							const int32 ConstraintIndex = ColorStart + Index;
							const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? StiffnessWarp[ConstraintIndex] : StiffnessNoMap;
							const FSolverReal ExpStiffnessWeftValue = StiffnessWeftHasWeightMap ? StiffnessWeft[ConstraintIndex] : StiffnessWeftNoMap;
							const FSolverReal ExpStiffnessBiasValue = StiffnessBiasHasWeightMap ? StiffnessBias[ConstraintIndex] : StiffnessBiasNoMap;
							const FSolverReal ExpBucklingValue = BucklingStiffnessHasWeightMap ? BucklingStiffnessWarp[ConstraintIndex] : BucklingStiffnessNoMap;
							const FSolverReal ExpBucklingWeftValue = BucklingStiffnessWeftHasWeightMap ? BucklingStiffnessWeft[ConstraintIndex] : BucklingStiffnessWeftNoMap;
							const FSolverReal ExpBucklingBiasValue = BucklingStiffnessBiasHasWeightMap ? BucklingStiffnessBias[ConstraintIndex] : BucklingStiffnessBiasNoMap;
							const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
							constexpr bool bDampingOnly = false;
							constexpr bool bElasticOnly = false;
							ApplyHelper<bDampingOnly, bElasticOnly>(Particles, Dt, ConstraintIndex,
								FSolverVec3(ExpStiffnessWeftValue, ExpStiffnessValue, ExpStiffnessBiasValue), FSolverVec3(ExpBucklingWeftValue, ExpBucklingValue, ExpBucklingBiasValue), DampingRatioValue);
						});
					}
					return;
				}

				for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
				{
					const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
					const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
					PhysicsParallelFor(ColorSize, [&](const int32 Index)
					{
						const int32 ConstraintIndex = ColorStart + Index;
						const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? StiffnessWarp[ConstraintIndex] : StiffnessNoMap;
						const FSolverReal ExpStiffnessWeftValue = StiffnessWeftHasWeightMap ? StiffnessWeft[ConstraintIndex] : StiffnessWeftNoMap;
						const FSolverReal ExpStiffnessBiasValue = StiffnessBiasHasWeightMap ? StiffnessBias[ConstraintIndex] : StiffnessBiasNoMap;
						const FSolverReal ExpBucklingValue = BucklingStiffnessHasWeightMap ? BucklingStiffnessWarp[ConstraintIndex] : BucklingStiffnessNoMap;
						const FSolverReal ExpBucklingWeftValue = BucklingStiffnessWeftHasWeightMap ? BucklingStiffnessWeft[ConstraintIndex] : BucklingStiffnessWeftNoMap;
						const FSolverReal ExpBucklingBiasValue = BucklingStiffnessBiasHasWeightMap ? BucklingStiffnessBias[ConstraintIndex] : BucklingStiffnessBiasNoMap;
						const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
						constexpr bool bDampingOnly = false;
						constexpr bool bElasticOnly = true;
						ApplyHelper<bDampingOnly, bElasticOnly>(Particles, Dt, ConstraintIndex,
							FSolverVec3(ExpStiffnessWeftValue, ExpStiffnessValue, ExpStiffnessBiasValue), FSolverVec3(ExpBucklingWeftValue, ExpBucklingValue, ExpBucklingBiasValue), DampingRatioValue);
					});
				}
			}
		}
	}
	else
	{
		// Single-threaded
		const FSolverReal StiffnessNoMap = (FSolverReal)Stiffness;
		const FSolverReal StiffnessWeftNoMap = (FSolverReal)StiffnessWeft;
		const FSolverReal StiffnessBiasNoMap = (FSolverReal)StiffnessBias;
		const FSolverReal BucklingStiffnessNoMap = (FSolverReal)BucklingStiffness;
		const FSolverReal BucklingStiffnessWeftNoMap = (FSolverReal)BucklingStiffnessWeft;
		const FSolverReal BucklingStiffnessBiasNoMap = (FSolverReal)BucklingStiffnessBias;
		const FSolverReal DampingNoMap = (FSolverReal)DampingRatio;
		if (DampingHasWeightMap || (FSolverReal)DampingRatio > 0)
		{
			if (bChaos_XPBDBending_SplitLambdaDamping)
			{
				for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
				{
					const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? StiffnessWarp[ConstraintIndex] : StiffnessNoMap;
					const FSolverReal ExpStiffnessWeftValue = StiffnessWeftHasWeightMap ? StiffnessWeft[ConstraintIndex] : StiffnessWeftNoMap;
					const FSolverReal ExpStiffnessBiasValue = StiffnessBiasHasWeightMap ? StiffnessBias[ConstraintIndex] : StiffnessBiasNoMap;
					const FSolverReal ExpBucklingValue = BucklingStiffnessHasWeightMap ? BucklingStiffnessWarp[ConstraintIndex] : BucklingStiffnessNoMap;
					const FSolverReal ExpBucklingWeftValue = BucklingStiffnessWeftHasWeightMap ? BucklingStiffnessWeft[ConstraintIndex] : BucklingStiffnessWeftNoMap;
					const FSolverReal ExpBucklingBiasValue = BucklingStiffnessBiasHasWeightMap ? BucklingStiffnessBias[ConstraintIndex] : BucklingStiffnessBiasNoMap;
					const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
					constexpr bool bDampingOnly = true;
					constexpr bool bElasticOnly = false;
					ApplyHelper<bDampingOnly, bElasticOnly>(Particles, Dt, ConstraintIndex,
						FSolverVec3(ExpStiffnessWeftValue, ExpStiffnessValue, ExpStiffnessBiasValue), FSolverVec3(ExpBucklingWeftValue, ExpBucklingValue, ExpBucklingBiasValue), DampingRatioValue);
				}
			}
			else
			{
				for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
				{
					const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? StiffnessWarp[ConstraintIndex] : StiffnessNoMap;
					const FSolverReal ExpStiffnessWeftValue = StiffnessWeftHasWeightMap ? StiffnessWeft[ConstraintIndex] : StiffnessWeftNoMap;
					const FSolverReal ExpStiffnessBiasValue = StiffnessBiasHasWeightMap ? StiffnessBias[ConstraintIndex] : StiffnessBiasNoMap;
					const FSolverReal ExpBucklingValue = BucklingStiffnessHasWeightMap ? BucklingStiffnessWarp[ConstraintIndex] : BucklingStiffnessNoMap;
					const FSolverReal ExpBucklingWeftValue = BucklingStiffnessWeftHasWeightMap ? BucklingStiffnessWeft[ConstraintIndex] : BucklingStiffnessWeftNoMap;
					const FSolverReal ExpBucklingBiasValue = BucklingStiffnessBiasHasWeightMap ? BucklingStiffnessBias[ConstraintIndex] : BucklingStiffnessBiasNoMap;
					const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
					constexpr bool bDampingOnly = false;
					constexpr bool bElasticOnly = false;
					ApplyHelper<bDampingOnly, bElasticOnly>(Particles, Dt, ConstraintIndex,
						FSolverVec3(ExpStiffnessWeftValue, ExpStiffnessValue, ExpStiffnessBiasValue), FSolverVec3(ExpBucklingWeftValue, ExpBucklingValue, ExpBucklingBiasValue), DampingRatioValue);
				}
				return;
			}
		}

		for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
		{
			const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? StiffnessWarp[ConstraintIndex] : StiffnessNoMap;
			const FSolverReal ExpStiffnessWeftValue = StiffnessWeftHasWeightMap ? StiffnessWeft[ConstraintIndex] : StiffnessWeftNoMap;
			const FSolverReal ExpStiffnessBiasValue = StiffnessBiasHasWeightMap ? StiffnessBias[ConstraintIndex] : StiffnessBiasNoMap;
			const FSolverReal ExpBucklingValue = BucklingStiffnessHasWeightMap ? BucklingStiffnessWarp[ConstraintIndex] : BucklingStiffnessNoMap;
			const FSolverReal ExpBucklingWeftValue = BucklingStiffnessWeftHasWeightMap ? BucklingStiffnessWeft[ConstraintIndex] : BucklingStiffnessWeftNoMap;
			const FSolverReal ExpBucklingBiasValue = BucklingStiffnessBiasHasWeightMap ? BucklingStiffnessBias[ConstraintIndex] : BucklingStiffnessBiasNoMap;
			const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
			constexpr bool bDampingOnly = false;
			constexpr bool bElasticOnly = true;
			ApplyHelper<bDampingOnly, bElasticOnly>(Particles, Dt, ConstraintIndex,
				FSolverVec3(ExpStiffnessWeftValue, ExpStiffnessValue, ExpStiffnessBiasValue), FSolverVec3(ExpBucklingWeftValue, ExpBucklingValue, ExpBucklingBiasValue), DampingRatioValue);
		}
	}
}
template CHAOS_API void FXPBDAnisotropicBendingConstraints::Apply(FSolverParticles& Particles, const FSolverReal Dt) const;
template CHAOS_API void FXPBDAnisotropicBendingConstraints::Apply(FSolverParticlesRange& Particles, const FSolverReal Dt) const;

void FXPBDAnisotropicBendingConstraints::ComputeGradTheta(const FSolverVec3& X0, const FSolverVec3& X1, const FSolverVec3& X2, const FSolverVec3& X3, const int32 Index, FSolverVec3& dThetadx, FSolverReal& Theta) 
{
	const FSolverVec3 E21 = X2 - X1;
	const FSolverReal Norme = E21.Length();
	const FSolverVec3 E10 = X1 - X0;
	const FSolverVec3 E20 = X2 - X0;
	const FSolverVec3 N0 = FSolverVec3::CrossProduct(E20, E10);
	const FSolverReal SquaredNorm0 = N0.SquaredLength();

	const FSolverVec3 E23 = X2 - X3;
	const FSolverVec3 E13 = X1 - X3;
	const FSolverVec3 N1 = FSolverVec3::CrossProduct(E13, E23); 
	const FSolverReal SquaredNorm1 = N1.SquaredLength();

	const FSolverVec3 N0CrossN1 = FSolverVec3::CrossProduct(N0, N1);
	Theta = FMath::Atan2(FSolverVec3::DotProduct(N0CrossN1, E21) / Norme, N0.Dot(N1)); 

	switch (Index)
		{
			case 0:
				dThetadx = -(Norme / SquaredNorm0) * N0;
				break;
			case 1:
				dThetadx = ((E20.Dot(E21)) / (Norme * SquaredNorm1)) * N1 + ((E23.Dot(E21)) / (Norme * SquaredNorm0)) * N0;
				break;
			case 2:
				dThetadx = -((E10.Dot(E21)) / (Norme * SquaredNorm1)) * N1 - ((E13.Dot(E21)) / (Norme * SquaredNorm0)) * N0;
				break;
			case 3:
				dThetadx = -(Norme / SquaredNorm1) * N1;
				break;
			default:
				break;
		}
}

void FXPBDAnisotropicBendingConstraints::AddAnisotropicBendingResidualAndHessian(const FSolverParticlesRange& Particles, const int32 ConstraintIndex, const int32 ConstraintIndexLocal, const FSolverReal Dt, TVec3<FSolverReal>& ParticleResidual, Chaos::PMatrix<FSolverReal, 3, 3>& ParticleHessian)
{
	const FSolverVec3 ExpStiffnessValue((FSolverReal)StiffnessWeft, (FSolverReal)StiffnessWarp, (FSolverReal)StiffnessBias);
	const FSolverVec3 ExpBucklingValue((FSolverReal)BucklingStiffnessWeft, (FSolverReal)BucklingStiffnessWarp, (FSolverReal)BucklingStiffnessBias);
	const FSolverReal DampingRatioValue = (FSolverReal)DampingRatio;
	const FSolverVec3& WarpWeftBiasBaseMultiplier = WarpWeftBiasBaseMultipliers[ConstraintIndex];

	const TVec4<int32>& Constraint = Constraints[ConstraintIndex];
	const int32 i1 = Constraint[0];
	const int32 i2 = Constraint[1];
	const int32 i3 = Constraint[2];
	const int32 i4 = Constraint[3];
	const FSolverVec3& P1 = Particles.P(Constraint[0]);
	const FSolverVec3& P2 = Particles.P(Constraint[1]);
	const FSolverVec3& P3 = Particles.P(Constraint[2]);
	const FSolverVec3& P4 = Particles.P(Constraint[3]);


	FSolverVec3 DThetaDx(0.f);
	FSolverReal Theta = 0.f;

	//TStaticArray<int32, 4> LocalIndexMap { 2, 1, 0, 3 };{ 2, 1, 0, 3 };
	constexpr int32 LocalIndexMap[] = { 2, 1, 0, 3 };
	const int32 ActualConstraintIndexLocal = LocalIndexMap[ConstraintIndexLocal];

	ComputeGradTheta(P3, P2, P1, P4, ActualConstraintIndexLocal, DThetaDx, Theta);

	const FSolverReal CAngle = Theta - RestAngles[ConstraintIndex];

	const FSolverReal BiphasicStiffnessValue = IsBuckled[ConstraintIndex] ?
		WarpWeftBiasBaseMultiplier.Dot(ExpBucklingValue) : WarpWeftBiasBaseMultiplier.Dot(ExpStiffnessValue);

	ParticleResidual -= Dt * Dt * BiphasicStiffnessValue * CAngle * DThetaDx;

	for (int32 Alpha = 0; Alpha < 3; Alpha++)
	{
		ParticleHessian.SetRow(Alpha, ParticleHessian.GetRow(Alpha) + Dt * Dt * BiphasicStiffnessValue * DThetaDx[Alpha] * DThetaDx);
	}
}

void FXPBDAnisotropicBendingConstraints::AddAnisotropicBendingResidualAndHessian(const FSolverParticles& Particles, const int32 ConstraintIndex, const int32 ConstraintIndexLocal, const FSolverReal Dt, TVec3<FSolverReal>& ParticleResidual, Chaos::PMatrix<FSolverReal, 3, 3>& ParticleHessian)
{
	AddAnisotropicBendingResidualAndHessian(FSolverParticlesRange(Particles), ConstraintIndex, ConstraintIndexLocal, Dt, ParticleResidual, ParticleHessian);
}


void FXPBDAnisotropicBendingConstraints::AddInternalForceDifferential(const FSolverParticlesRange& InParticles, const TArray<TVector<FSolverReal, 3>>& DeltaParticles, TArray<TVector<FSolverReal, 3>>& Ndf)
{
	const FSolverVec3 ExpStiffnessValue((FSolverReal)StiffnessWeft, (FSolverReal)StiffnessWarp, (FSolverReal)StiffnessBias);
	const FSolverVec3 ExpBucklingValue((FSolverReal)BucklingStiffnessWeft, (FSolverReal)BucklingStiffnessWarp, (FSolverReal)BucklingStiffnessBias);
	ensure(Ndf.Num() == InParticles.Size());

	const int32 ConstraintColorNum = ConstraintsPerColorStartIndex.Num() - 1;

	for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
	{
		const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
		const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
		PhysicsParallelFor(ColorSize, [&](const int32 Index)
			{
				const int32 ConstraintIndex = ColorStart + Index;
				const TVec4<int32>& Constraint = Constraints[ConstraintIndex];
				const FSolverVec3& P1 = InParticles.P(Constraint[0]);
				const FSolverVec3& P2 = InParticles.P(Constraint[1]);
				const FSolverVec3& P3 = InParticles.P(Constraint[2]);
				const FSolverVec3& P4 = InParticles.P(Constraint[3]);
				const FSolverVec3& WarpWeftBiasBaseMultiplier = WarpWeftBiasBaseMultipliers[ConstraintIndex];
				FSolverReal Theta = 0.f;

				TArray<FSolverVec3> AllDThetaDx;
				AllDThetaDx.Init(FSolverVec3(0.f), 4);

				constexpr int32 LocalIndexMap[] = { 2, 1, 0, 3 };
				for (int32 LocalIdx = 0; LocalIdx < 4; LocalIdx++)
				{
					const int32 ActualConstraintIndexLocal = LocalIndexMap[LocalIdx];
					ComputeGradTheta(P3, P2, P1, P4, ActualConstraintIndexLocal, AllDThetaDx[LocalIdx], Theta);
				}

				const FSolverReal BiphasicStiffnessValue = IsBuckled[ConstraintIndex] ?
					WarpWeftBiasBaseMultiplier.Dot(ExpBucklingValue) : WarpWeftBiasBaseMultiplier.Dot(ExpStiffnessValue);

				FSolverReal DeltaC = 0.f;

				for (int32 LocalIdx = 0; LocalIdx < 4; LocalIdx++)
				{
					for (int32 Alpha = 0; Alpha < 3; Alpha++)
					{
						DeltaC += AllDThetaDx[LocalIdx][Alpha] * DeltaParticles[Constraint[LocalIdx]][Alpha];
					}
				}

				for (int32 LocalIdx = 0; LocalIdx < 4; LocalIdx++)
				{
					Ndf[Constraint[LocalIdx]] += BiphasicStiffnessValue * DeltaC * AllDThetaDx[LocalIdx];
				}

		});
	}
}

void FXPBDAnisotropicBendingConstraints::AddInternalForceDifferential(const FSolverParticles& InParticles, const TArray<TVector<FSolverReal, 3>>& DeltaParticles, TArray<TVector<FSolverReal, 3>>& Ndf)
{
	AddInternalForceDifferential(FSolverParticlesRange(InParticles), DeltaParticles, Ndf);
}




}  // End namespace Chaos::Softs
