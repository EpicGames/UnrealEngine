// Copyright Epic Games, Inc. All Rights Reserved.

#define EXPLICIT_VECTOR4 1

#include "Math/Vector.isph"
#include "Chaos/PBDSofts.isph"
#include "Chaos/XPBDBendingElement.isph"

export void ApplyXPBDAnisotropicBendingConstraints(uniform FVector4f PandInvM[],
									const uniform int32 ConstraintIndex1[],
									const uniform int32 ConstraintIndex2[],
									const uniform int32 ConstraintIndex3[],
									const uniform int32 ConstraintIndex4[],
									const uniform float RestAngles[],
									const uniform bool IsBuckled[],
									const uniform FVector3f WarpWeftBiasBaseMultiplers[],
									uniform float Lambdas[],
									const uniform float Dt,
									const uniform FVector3f& Stiffness,
									const uniform FVector3f& BucklingStiffness,
									const uniform int32 NumConstraints)
{

	foreach(i = 0 ... NumConstraints)
	{
		const varying int32 i1 = ConstraintIndex1[i];
		const varying int32 i2 = ConstraintIndex2[i];
		const varying int32 i3 = ConstraintIndex3[i];
		const varying int32 i4 = ConstraintIndex4[i];

		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[i1]);
		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[i2]);
		const varying FVector4f PandInvM3 = VectorGather(&PandInvM[i3]);
		const varying FVector4f PandInvM4 = VectorGather(&PandInvM[i4]);

		varying FVector3f P1, P2, P3, P4;
		varying float InvM1, InvM2, InvM3, InvM4;

		UnzipPandInvM(PandInvM1, P1, InvM1);
		UnzipPandInvM(PandInvM2, P2, InvM2);
		UnzipPandInvM(PandInvM3, P3, InvM3);
		UnzipPandInvM(PandInvM4, P4, InvM4);
		
		const varying FVector3f WarpWeftBiasBaseMultiplier = VectorLoad(&WarpWeftBiasBaseMultiplers[extract(i,0)]);
		const varying float StiffnessValue = select(IsBuckled[i], VectorDot(WarpWeftBiasBaseMultiplier, BucklingStiffness), VectorDot(WarpWeftBiasBaseMultiplier, Stiffness));
		
		const varying float Lambda = Lambdas[i];

		varying FVector3f Delta1, Delta2, Delta3, Delta4;
		const varying float DLambda = GetXPBDBendingElementDeltas(P1, P2, P3, P4, InvM1, InvM2, InvM3, InvM4, Dt, RestAngles[i], Lambda, StiffnessValue, Delta1, Delta2, Delta3, Delta4);

		Lambdas[i] = Lambda + DLambda;
		VectorScatter(&PandInvM[i1], SetVector4(P1 + Delta1, InvM1));
		VectorScatter(&PandInvM[i2], SetVector4(P2 + Delta2, InvM2));
		VectorScatter(&PandInvM[i3], SetVector4(P3 + Delta3, InvM3));
		VectorScatter(&PandInvM[i4], SetVector4(P4 + Delta4, InvM4));
	}
}

export void ApplyXPBDAnisotropicBendingConstraintsWithDamping(uniform FVector4f PandInvM[],
									const uniform FVector3f X1Array[],
									const uniform FVector3f X2Array[],
									const uniform FVector3f X3Array[],
									const uniform FVector3f X4Array[],
									const uniform int32 ConstraintIndex1[],
									const uniform int32 ConstraintIndex2[],
									const uniform int32 ConstraintIndex3[],
									const uniform int32 ConstraintIndex4[],
									const uniform float RestAngles[],
									const uniform bool IsBuckled[],
									const uniform FVector3f WarpWeftBiasBaseMultiplers[],
									uniform float Lambdas[],
									const uniform float Dt,
									const uniform FVector3f& Stiffness,
									const uniform FVector3f& BucklingStiffness,
									const uniform float DampingRatio,
									const uniform int32 NumConstraints)
{

	foreach(i = 0 ... NumConstraints)
	{
		const varying int32 i1 = ConstraintIndex1[i];
		const varying int32 i2 = ConstraintIndex2[i];
		const varying int32 i3 = ConstraintIndex3[i];
		const varying int32 i4 = ConstraintIndex4[i];
		
		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[i1]);
		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[i2]);
		const varying FVector4f PandInvM3 = VectorGather(&PandInvM[i3]);
		const varying FVector4f PandInvM4 = VectorGather(&PandInvM[i4]);

		varying FVector3f P1, P2, P3, P4;
		varying float M1, M2, M3, M4;

		UnzipPandInvM(PandInvM1, P1, M1);
		UnzipPandInvM(PandInvM2, P2, M2);
		UnzipPandInvM(PandInvM3, P3, M3);
		UnzipPandInvM(PandInvM4, P4, M4);
		
		const uniform int32 i0 = extract(i,0);

		const varying FVector3f X1 = VectorLoad(&X1Array[i0]);
		const varying FVector3f X2 = VectorLoad(&X2Array[i0]);
		const varying FVector3f X3 = VectorLoad(&X3Array[i0]);
		const varying FVector3f X4 = VectorLoad(&X4Array[i0]);
		
		const varying FVector3f WarpWeftBiasBaseMultiplier = VectorLoad(&WarpWeftBiasBaseMultiplers[i0]);
		const varying float StiffnessValue = select(IsBuckled[i], VectorDot(WarpWeftBiasBaseMultiplier, BucklingStiffness), VectorDot(WarpWeftBiasBaseMultiplier, Stiffness));
		const varying float Lambda = Lambdas[i];

		varying FVector3f Delta1, Delta2, Delta3, Delta4;
		const varying float DLambda = GetXPBDBendingElementDeltasWithDamping(P1, P2, P3, P4, X1, X2, X3, X4, M1, M2, M3, M4, Dt, RestAngles[i], Lambda, StiffnessValue, DampingRatio, Delta1, Delta2, Delta3, Delta4);

		Lambdas[i] = Lambda + DLambda;

		VectorScatter(&PandInvM[i1], SetVector4(P1 + Delta1, M1));
		VectorScatter(&PandInvM[i2], SetVector4(P2 + Delta2, M2));
		VectorScatter(&PandInvM[i3], SetVector4(P3 + Delta3, M3));
		VectorScatter(&PandInvM[i4], SetVector4(P4 + Delta4, M4));
	}
}

export void ApplyXPBDAnisotropicBendingDampingConstraints(uniform FVector4f PandInvM[],
									const uniform FVector3f X1Array[],
									const uniform FVector3f X2Array[],
									const uniform FVector3f X3Array[],
									const uniform FVector3f X4Array[],
									const uniform int32 ConstraintIndex1[],
									const uniform int32 ConstraintIndex2[],
									const uniform int32 ConstraintIndex3[],
									const uniform int32 ConstraintIndex4[],
									const uniform float RestAngles[],
									const uniform bool IsBuckled[],
									const uniform FVector3f WarpWeftBiasBaseMultiplers[],
									uniform float Lambdas[],
									const uniform float Dt,
									const uniform FVector3f& Stiffness,
									const uniform FVector3f& BucklingStiffness,
									const uniform float DampingRatio,
									const uniform int32 NumConstraints)
{

	foreach(i = 0 ... NumConstraints)
	{
		const varying int32 i1 = ConstraintIndex1[i];
		const varying int32 i2 = ConstraintIndex2[i];
		const varying int32 i3 = ConstraintIndex3[i];
		const varying int32 i4 = ConstraintIndex4[i];
		
		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[i1]);
		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[i2]);
		const varying FVector4f PandInvM3 = VectorGather(&PandInvM[i3]);
		const varying FVector4f PandInvM4 = VectorGather(&PandInvM[i4]);

		varying FVector3f P1, P2, P3, P4;
		varying float M1, M2, M3, M4;

		UnzipPandInvM(PandInvM1, P1, M1);
		UnzipPandInvM(PandInvM2, P2, M2);
		UnzipPandInvM(PandInvM3, P3, M3);
		UnzipPandInvM(PandInvM4, P4, M4);
		
		const uniform int32 i0 = extract(i,0);

		const varying FVector3f X1 = VectorLoad(&X1Array[i0]);
		const varying FVector3f X2 = VectorLoad(&X2Array[i0]);
		const varying FVector3f X3 = VectorLoad(&X3Array[i0]);
		const varying FVector3f X4 = VectorLoad(&X4Array[i0]);
		
		const varying FVector3f WarpWeftBiasBaseMultiplier = VectorLoad(&WarpWeftBiasBaseMultiplers[i0]);
		const varying float StiffnessValue = select(IsBuckled[i], VectorDot(WarpWeftBiasBaseMultiplier, BucklingStiffness), VectorDot(WarpWeftBiasBaseMultiplier, Stiffness));
		const varying float Lambda = Lambdas[i];

		varying FVector3f Delta1, Delta2, Delta3, Delta4;
		const varying float DLambda = GetXPBDBendingElementDampingDeltas(P1, P2, P3, P4, X1, X2, X3, X4, M1, M2, M3, M4, Dt, RestAngles[i], Lambda, StiffnessValue, DampingRatio, Delta1, Delta2, Delta3, Delta4);

		Lambdas[i] = Lambda + DLambda;
		VectorScatter(&PandInvM[i1], SetVector4(P1 + Delta1, M1));
		VectorScatter(&PandInvM[i2], SetVector4(P2 + Delta2, M2));
		VectorScatter(&PandInvM[i3], SetVector4(P3 + Delta3, M3));
		VectorScatter(&PandInvM[i4], SetVector4(P4 + Delta4, M4));
	}
}

export void ApplyXPBDAnisotropicBendingConstraintsWithMaps(uniform FVector4f PandInvM[],
											const uniform int32 ConstraintIndex1[],
											const uniform int32 ConstraintIndex2[],
											const uniform int32 ConstraintIndex3[],
											const uniform int32 ConstraintIndex4[],
											const uniform float RestAngles[],
											const uniform bool IsBuckled[],
											const uniform FVector3f WarpWeftBiasBaseMultiplers[],
											uniform float Lambdas[],
											const uniform float Dt,
											const uniform bool StiffnessHasMap,
											const uniform FVector2f& StiffnessOffsetRange,
											const uniform float StiffnessMapValues[],
											const uniform bool StiffnessWeftHasMap,
											const uniform FVector2f& StiffnessWeftOffsetRange,
											const uniform float StiffnessWeftMapValues[],
											const uniform bool StiffnessBiasHasMap,
											const uniform FVector2f& StiffnessBiasOffsetRange,
											const uniform float StiffnessBiasMapValues[],
											const uniform bool BucklingStiffnessHasMap,
											const uniform FVector2f& BucklingStiffnessOffsetRange,
											const uniform float BucklingStiffnessMapValues[],
											const uniform bool BucklingStiffnessWeftHasMap,
											const uniform FVector2f& BucklingStiffnessWeftOffsetRange,
											const uniform float BucklingStiffnessWeftMapValues[],
											const uniform bool BucklingStiffnessBiasHasMap,
											const uniform FVector2f& BucklingStiffnessBiasOffsetRange,
											const uniform float BucklingStiffnessBiasMapValues[],
											const uniform int32 NumConstraints)
{

	foreach(i = 0 ... NumConstraints)
	{
		const varying int32 i1 = ConstraintIndex1[i];
		const varying int32 i2 = ConstraintIndex2[i];
		const varying int32 i3 = ConstraintIndex3[i];
		const varying int32 i4 = ConstraintIndex4[i];

		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[i2]);
		const varying FVector4f PandInvM3 = VectorGather(&PandInvM[i3]);
		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[i1]);
		const varying FVector4f PandInvM4 = VectorGather(&PandInvM[i4]);

		varying FVector3f P1, P2, P3, P4;
		varying float M1, M2, M3, M4;

		UnzipPandInvM(PandInvM1, P1, M1);
		UnzipPandInvM(PandInvM2, P2, M2);
		UnzipPandInvM(PandInvM3, P3, M3);
		UnzipPandInvM(PandInvM4, P4, M4);

		const varying float Stiffness = StiffnessHasMap ? StiffnessOffsetRange.V[0] + StiffnessMapValues[i]*StiffnessOffsetRange.V[1] : StiffnessOffsetRange.V[0];
		const varying float StiffnessWeft = StiffnessWeftHasMap ? StiffnessWeftOffsetRange.V[0] + StiffnessWeftMapValues[i]*StiffnessWeftOffsetRange.V[1] : StiffnessWeftOffsetRange.V[0];
		const varying float StiffnessBias = StiffnessBiasHasMap ? StiffnessBiasOffsetRange.V[0] + StiffnessBiasMapValues[i]*StiffnessBiasOffsetRange.V[1] : StiffnessBiasOffsetRange.V[0];

		const varying FVector3f StiffnessVector = {{StiffnessWeft, Stiffness, StiffnessBias}};
		
		const varying float BucklingStiffness = BucklingStiffnessHasMap ? BucklingStiffnessOffsetRange.V[0] + BucklingStiffnessMapValues[i]*BucklingStiffnessOffsetRange.V[1] : BucklingStiffnessOffsetRange.V[0];
		const varying float BucklingStiffnessWeft = BucklingStiffnessWeftHasMap ? BucklingStiffnessWeftOffsetRange.V[0] + BucklingStiffnessWeftMapValues[i]*BucklingStiffnessWeftOffsetRange.V[1] : BucklingStiffnessWeftOffsetRange.V[0];
		const varying float BucklingStiffnessBias = BucklingStiffnessBiasHasMap ? BucklingStiffnessBiasOffsetRange.V[0] + BucklingStiffnessBiasMapValues[i]*BucklingStiffnessBiasOffsetRange.V[1] : BucklingStiffnessBiasOffsetRange.V[0];
		
		const varying FVector3f BucklingStiffnessVector = {{BucklingStiffnessWeft, BucklingStiffness, BucklingStiffnessBias}};
		
		const varying FVector3f WarpWeftBiasBaseMultiplier = VectorLoad(&WarpWeftBiasBaseMultiplers[extract(i,0)]);
		const varying float StiffnessValue = select(IsBuckled[i], VectorDot(WarpWeftBiasBaseMultiplier, BucklingStiffnessVector), VectorDot(WarpWeftBiasBaseMultiplier, StiffnessVector));
		
		const varying float Lambda = Lambdas[i];

		varying FVector3f Delta1, Delta2, Delta3, Delta4;
		const varying float DLambda = GetXPBDBendingElementDeltas(P1, P2, P3, P4, M1, M2, M3, M4, Dt, RestAngles[i], Lambda, StiffnessValue, Delta1, Delta2, Delta3, Delta4);

		Lambdas[i] = Lambda + DLambda;
		
		VectorScatter(&PandInvM[i1], SetVector4(P1 + Delta1, M1));
		VectorScatter(&PandInvM[i2], SetVector4(P2 + Delta2, M2));
		VectorScatter(&PandInvM[i3], SetVector4(P3 + Delta3, M3));
		VectorScatter(&PandInvM[i4], SetVector4(P4 + Delta4, M4));
	}
}

export void ApplyXPBDAnisotropicBendingConstraintsWithDampingAndMaps(uniform FVector4f PandInvM[],
											const uniform FVector3f X1Array[],
											const uniform FVector3f X2Array[],
											const uniform FVector3f X3Array[],
											const uniform FVector3f X4Array[],
											const uniform int32 ConstraintIndex1[],
											const uniform int32 ConstraintIndex2[],
											const uniform int32 ConstraintIndex3[],
											const uniform int32 ConstraintIndex4[],
											const uniform float RestAngles[],
											const uniform bool IsBuckled[],
											const uniform FVector3f WarpWeftBiasBaseMultiplers[],
											uniform float Lambdas[],
											const uniform float Dt,
											const uniform bool StiffnessHasMap,
											const uniform FVector2f& StiffnessOffsetRange,
											const uniform float StiffnessMapValues[],
											const uniform bool StiffnessWeftHasMap,
											const uniform FVector2f& StiffnessWeftOffsetRange,
											const uniform float StiffnessWeftMapValues[],
											const uniform bool StiffnessBiasHasMap,
											const uniform FVector2f& StiffnessBiasOffsetRange,
											const uniform float StiffnessBiasMapValues[],
											const uniform bool BucklingStiffnessHasMap,
											const uniform FVector2f& BucklingStiffnessOffsetRange,
											const uniform float BucklingStiffnessMapValues[],
											const uniform bool BucklingStiffnessWeftHasMap,
											const uniform FVector2f& BucklingStiffnessWeftOffsetRange,
											const uniform float BucklingStiffnessWeftMapValues[],
											const uniform bool BucklingStiffnessBiasHasMap,
											const uniform FVector2f& BucklingStiffnessBiasOffsetRange,
											const uniform float BucklingStiffnessBiasMapValues[],
											const uniform bool DampingHasMap,
											const uniform FVector2f& DampingOffsetRange,
											const uniform float DampingMapValues[],
											const uniform int32 NumConstraints)
{

	foreach(i = 0 ... NumConstraints)
	{
		const varying int32 i1 = ConstraintIndex1[i];
		const varying int32 i2 = ConstraintIndex2[i];
		const varying int32 i3 = ConstraintIndex3[i];
		const varying int32 i4 = ConstraintIndex4[i];

		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[i2]);
		const varying FVector4f PandInvM3 = VectorGather(&PandInvM[i3]);
		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[i1]);
		const varying FVector4f PandInvM4 = VectorGather(&PandInvM[i4]);

		varying FVector3f P1, P2, P3, P4;
		varying float M1, M2, M3, M4;

		UnzipPandInvM(PandInvM1, P1, M1);
		UnzipPandInvM(PandInvM2, P2, M2);
		UnzipPandInvM(PandInvM3, P3, M3);
		UnzipPandInvM(PandInvM4, P4, M4);

		const varying float Stiffness = StiffnessHasMap ? StiffnessOffsetRange.V[0] + StiffnessMapValues[i]*StiffnessOffsetRange.V[1] : StiffnessOffsetRange.V[0];
		const varying float StiffnessWeft = StiffnessWeftHasMap ? StiffnessWeftOffsetRange.V[0] + StiffnessWeftMapValues[i]*StiffnessWeftOffsetRange.V[1] : StiffnessWeftOffsetRange.V[0];
		const varying float StiffnessBias = StiffnessBiasHasMap ? StiffnessBiasOffsetRange.V[0] + StiffnessBiasMapValues[i]*StiffnessBiasOffsetRange.V[1] : StiffnessBiasOffsetRange.V[0];

		const varying FVector3f StiffnessVector = {{StiffnessWeft, Stiffness, StiffnessBias}};
		
		const varying float BucklingStiffness = BucklingStiffnessHasMap ? BucklingStiffnessOffsetRange.V[0] + BucklingStiffnessMapValues[i]*BucklingStiffnessOffsetRange.V[1] : BucklingStiffnessOffsetRange.V[0];
		const varying float BucklingStiffnessWeft = BucklingStiffnessWeftHasMap ? BucklingStiffnessWeftOffsetRange.V[0] + BucklingStiffnessWeftMapValues[i]*BucklingStiffnessWeftOffsetRange.V[1] : BucklingStiffnessWeftOffsetRange.V[0];
		const varying float BucklingStiffnessBias = BucklingStiffnessBiasHasMap ? BucklingStiffnessBiasOffsetRange.V[0] + BucklingStiffnessBiasMapValues[i]*BucklingStiffnessBiasOffsetRange.V[1] : BucklingStiffnessBiasOffsetRange.V[0];
		
		const varying FVector3f BucklingStiffnessVector = {{BucklingStiffnessWeft, BucklingStiffness, BucklingStiffnessBias}};
		
		const varying float DampingRatio = DampingHasMap ? DampingOffsetRange.V[0] + DampingMapValues[i]*DampingOffsetRange.V[1] : DampingOffsetRange.V[0];
		
		const uniform int32 i0 = extract(i,0);

		const varying FVector3f X1 = VectorLoad(&X1Array[i0]);
		const varying FVector3f X2 = VectorLoad(&X2Array[i0]);
		const varying FVector3f X3 = VectorLoad(&X3Array[i0]);
		const varying FVector3f X4 = VectorLoad(&X4Array[i0]);
		
		const varying FVector3f WarpWeftBiasBaseMultiplier = VectorLoad(&WarpWeftBiasBaseMultiplers[i0]);
		const varying float StiffnessValue = select(IsBuckled[i], VectorDot(WarpWeftBiasBaseMultiplier, BucklingStiffnessVector), VectorDot(WarpWeftBiasBaseMultiplier, StiffnessVector));
		
		const varying float Lambda = Lambdas[i];

		varying FVector3f Delta1, Delta2, Delta3, Delta4;
		const varying float DLambda = GetXPBDBendingElementDeltasWithDamping(P1, P2, P3, P4, X1, X2, X3, X4, M1, M2, M3, M4, Dt, RestAngles[i], Lambda, StiffnessValue, DampingRatio, Delta1, Delta2, Delta3, Delta4);

		Lambdas[i] = Lambda + DLambda;

		VectorScatter(&PandInvM[i1], SetVector4(P1 + Delta1, M1));
		VectorScatter(&PandInvM[i2], SetVector4(P2 + Delta2, M2));
		VectorScatter(&PandInvM[i3], SetVector4(P3 + Delta3, M3));
		VectorScatter(&PandInvM[i4], SetVector4(P4 + Delta4, M4));
	}
}

export void ApplyXPBDAnisotropicBendingDampingConstraintsWithMaps(uniform FVector4f PandInvM[],
											const uniform FVector3f X1Array[],
											const uniform FVector3f X2Array[],
											const uniform FVector3f X3Array[],
											const uniform FVector3f X4Array[],
											const uniform int32 ConstraintIndex1[],
											const uniform int32 ConstraintIndex2[],
											const uniform int32 ConstraintIndex3[],
											const uniform int32 ConstraintIndex4[],
											const uniform float RestAngles[],
											const uniform bool IsBuckled[],
											const uniform FVector3f WarpWeftBiasBaseMultiplers[],
											uniform float Lambdas[],
											const uniform float Dt,
											const uniform bool StiffnessHasMap,
											const uniform FVector2f& StiffnessOffsetRange,
											const uniform float StiffnessMapValues[],
											const uniform bool StiffnessWeftHasMap,
											const uniform FVector2f& StiffnessWeftOffsetRange,
											const uniform float StiffnessWeftMapValues[],
											const uniform bool StiffnessBiasHasMap,
											const uniform FVector2f& StiffnessBiasOffsetRange,
											const uniform float StiffnessBiasMapValues[],
											const uniform bool BucklingStiffnessHasMap,
											const uniform FVector2f& BucklingStiffnessOffsetRange,
											const uniform float BucklingStiffnessMapValues[],
											const uniform bool BucklingStiffnessWeftHasMap,
											const uniform FVector2f& BucklingStiffnessWeftOffsetRange,
											const uniform float BucklingStiffnessWeftMapValues[],
											const uniform bool BucklingStiffnessBiasHasMap,
											const uniform FVector2f& BucklingStiffnessBiasOffsetRange,
											const uniform float BucklingStiffnessBiasMapValues[],
											const uniform bool DampingHasMap,
											const uniform FVector2f& DampingOffsetRange,
											const uniform float DampingMapValues[],
											const uniform int32 NumConstraints)
{

	foreach(i = 0 ... NumConstraints)
	{
		const varying int32 i1 = ConstraintIndex1[i];
		const varying int32 i2 = ConstraintIndex2[i];
		const varying int32 i3 = ConstraintIndex3[i];
		const varying int32 i4 = ConstraintIndex4[i];

		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[i2]);
		const varying FVector4f PandInvM3 = VectorGather(&PandInvM[i3]);
		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[i1]);
		const varying FVector4f PandInvM4 = VectorGather(&PandInvM[i4]);

		varying FVector3f P1, P2, P3, P4;
		varying float M1, M2, M3, M4;

		UnzipPandInvM(PandInvM1, P1, M1);
		UnzipPandInvM(PandInvM2, P2, M2);
		UnzipPandInvM(PandInvM3, P3, M3);
		UnzipPandInvM(PandInvM4, P4, M4);

		const varying float Stiffness = StiffnessHasMap ? StiffnessOffsetRange.V[0] + StiffnessMapValues[i]*StiffnessOffsetRange.V[1] : StiffnessOffsetRange.V[0];
		const varying float StiffnessWeft = StiffnessWeftHasMap ? StiffnessWeftOffsetRange.V[0] + StiffnessWeftMapValues[i]*StiffnessWeftOffsetRange.V[1] : StiffnessWeftOffsetRange.V[0];
		const varying float StiffnessBias = StiffnessBiasHasMap ? StiffnessBiasOffsetRange.V[0] + StiffnessBiasMapValues[i]*StiffnessBiasOffsetRange.V[1] : StiffnessBiasOffsetRange.V[0];

		const varying FVector3f StiffnessVector = {{StiffnessWeft, Stiffness, StiffnessBias}};
		
		const varying float BucklingStiffness = BucklingStiffnessHasMap ? BucklingStiffnessOffsetRange.V[0] + BucklingStiffnessMapValues[i]*BucklingStiffnessOffsetRange.V[1] : BucklingStiffnessOffsetRange.V[0];
		const varying float BucklingStiffnessWeft = BucklingStiffnessWeftHasMap ? BucklingStiffnessWeftOffsetRange.V[0] + BucklingStiffnessWeftMapValues[i]*BucklingStiffnessWeftOffsetRange.V[1] : BucklingStiffnessWeftOffsetRange.V[0];
		const varying float BucklingStiffnessBias = BucklingStiffnessBiasHasMap ? BucklingStiffnessBiasOffsetRange.V[0] + BucklingStiffnessBiasMapValues[i]*BucklingStiffnessBiasOffsetRange.V[1] : BucklingStiffnessBiasOffsetRange.V[0];
		
		const varying FVector3f BucklingStiffnessVector = {{BucklingStiffnessWeft, BucklingStiffness, BucklingStiffnessBias}};
		
		const varying float DampingRatio = DampingHasMap ? DampingOffsetRange.V[0] + DampingMapValues[i]*DampingOffsetRange.V[1] : DampingOffsetRange.V[0];

		const uniform int32 i0 = extract(i,0);

		const varying FVector3f X1 = VectorLoad(&X1Array[i0]);
		const varying FVector3f X2 = VectorLoad(&X2Array[i0]);
		const varying FVector3f X3 = VectorLoad(&X3Array[i0]);
		const varying FVector3f X4 = VectorLoad(&X4Array[i0]);
		
		const varying FVector3f WarpWeftBiasBaseMultiplier = VectorLoad(&WarpWeftBiasBaseMultiplers[i0]);
		const varying float StiffnessValue = select(IsBuckled[i], VectorDot(WarpWeftBiasBaseMultiplier, BucklingStiffnessVector), VectorDot(WarpWeftBiasBaseMultiplier, StiffnessVector));
		
		const varying float Lambda = Lambdas[i];

		varying FVector3f Delta1, Delta2, Delta3, Delta4;
		const varying float DLambda = GetXPBDBendingElementDampingDeltas(P1, P2, P3, P4, X1, X2, X3, X4, M1, M2, M3, M4, Dt, RestAngles[i], Lambda, StiffnessValue, DampingRatio, Delta1, Delta2, Delta3, Delta4);

		Lambdas[i] = Lambda + DLambda;
		
		VectorScatter(&PandInvM[i1], SetVector4(P1 + Delta1, M1));
		VectorScatter(&PandInvM[i2], SetVector4(P2 + Delta2, M2));
		VectorScatter(&PandInvM[i3], SetVector4(P3 + Delta3, M3));
		VectorScatter(&PandInvM[i4], SetVector4(P4 + Delta4, M4));
	}
}
