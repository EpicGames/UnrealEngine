// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/XPBDAnisotropicSpringConstraints.h"
#include "Chaos/XPBDSpringConstraints.h"
#include "Chaos/GraphColoring.h"
#include "Chaos/Framework/Parallel.h"
#include "Chaos/SoftsSpring.h"
#include "Chaos/TriangleMesh.h"
#include "ChaosStats.h"
#include "XPBDInternal.h"

#if INTEL_ISPC
#include "XPBDAnisotropicSpringConstraints.ispc.generated.h"
#endif

DECLARE_CYCLE_STAT(TEXT("Chaos XPBD Anisotropic Spring Constraint"), STAT_XPBD_AnisoSpring, STATGROUP_Chaos);

#if INTEL_ISPC && !UE_BUILD_SHIPPING
static_assert(sizeof(ispc::FVector4f) == sizeof(Chaos::Softs::FPAndInvM), "sizeof(ispc::FVector4f) != sizeof(Chaos::Softs::FPAndInvM");
static_assert(sizeof(ispc::FVector3f) == sizeof(Chaos::Softs::FSolverVec3), "sizeof(ispc::FVector3f) == sizeof(Chaos::Softs::FSolverVec3)");
static_assert(sizeof(ispc::FIntVector2) == sizeof(Chaos::TVec2<int32>), "sizeof(ispc::FIntVector2) != sizeof(Chaos::TVec2<int32>)");
#endif

namespace Chaos::Softs {

// @todo(chaos): the parallel threshold (or decision to run parallel) should probably be owned by the solver and passed to the constraint container
extern int32 Chaos_XPBDSpring_ParallelConstraintCount;

namespace Private
{
static void UpdateDists(const TArray<FSolverReal>& BaseDists, const TArray<FSolverVec2>& WarpWeftScaleBaseMultipliers, const FPBDWeightMap& WarpScale, const FPBDWeightMap& WeftScale, TArray<FSolverReal>& Dists)
{
	const bool WarpScaleHasWeightMap = WarpScale.HasWeightMap();
	const bool WeftScaleHasWeightMap = WeftScale.HasWeightMap();
	const FSolverReal WarpScaleNoMap = (FSolverReal)WarpScale;
	const FSolverReal WeftScaleNoMap = (FSolverReal)WeftScale;
	for (int32 ConstraintIndex = 0; ConstraintIndex < BaseDists.Num(); ++ConstraintIndex)
	{
		const FSolverReal WarpScaleValue = WarpScaleHasWeightMap ? WarpScale[ConstraintIndex] : WarpScaleNoMap;
		const FSolverReal WeftScaleValue = WeftScaleHasWeightMap ? WeftScale[ConstraintIndex] : WeftScaleNoMap;

		Dists[ConstraintIndex] = BaseDists[ConstraintIndex] * FMath::Sqrt(FMath::Square(WeftScaleValue * WarpWeftScaleBaseMultipliers[ConstraintIndex][0]) + FMath::Square(WarpScaleValue * WarpWeftScaleBaseMultipliers[ConstraintIndex][1]));
	}
}
}

FXPBDAnisotropicEdgeSpringConstraints::FXPBDAnisotropicEdgeSpringConstraints(
	const FSolverParticlesRange& Particles,
	const FTriangleMesh& TriangleMesh,
	const TArray<TVec3<FVec2f>>& FaceVertexPatternPositions,
	bool bUse3dRestLengths,
	bool SavePreResizedSimPosition3D,
	const TArray<FVector3f>& PreResizedSimPosition3D,
	FSolverReal MeshScale,
	const TConstArrayView<FRealSingle>& StiffnessWarpMultipliers,
	const TConstArrayView<FRealSingle>& StiffnessWeftMultipliers,
	const TConstArrayView<FRealSingle>& StiffnessBiasMultipliers,
	const TConstArrayView<FRealSingle>& DampingMultipliers,
	const TConstArrayView<FRealSingle>& WarpScaleMultipliers,
	const TConstArrayView<FRealSingle>& WeftScaleMultipliers,
	const FSolverVec2& InStiffnessWarp,
	const FSolverVec2& InStiffnessWeft,
	const FSolverVec2& InStiffnessBias,
	const FSolverVec2& InDampingRatio,
	const FSolverVec2& InWarpScale,
	const FSolverVec2& InWeftScale)
	: Base(Particles,
		TriangleMesh.GetElements(),
		TConstArrayView<FRealSingle>(), // We don't use base stiffness maps
		InStiffnessWarp,
		true /*bTrimKinematicConstraints*/,
		MaxStiffness)
	, StiffnessWarp(InStiffnessWarp.ClampAxes(0, MaxStiffness),
		StiffnessWarpMultipliers,
		TConstArrayView<TVec2<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, StiffnessWeft(InStiffnessWeft.ClampAxes(0, MaxStiffness),
		StiffnessWeftMultipliers,
		TConstArrayView<TVec2<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, StiffnessBias(InStiffnessBias.ClampAxes(0, MaxStiffness),
		StiffnessBiasMultipliers,
		TConstArrayView<TVec2<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, DampingRatio(
		InDampingRatio.ClampAxes(MinDampingRatio, MaxDampingRatio),
		DampingMultipliers,
		TConstArrayView<TVec2<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, WarpScale(
		InWarpScale.ClampAxes(MinWarpWeftScale, MaxWarpWeftScale),
		WarpScaleMultipliers,
		TConstArrayView<TVec2<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, WeftScale(
		InWeftScale.ClampAxes(MinWarpWeftScale, MaxWarpWeftScale),
		WeftScaleMultipliers,
		TConstArrayView<TVec2<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
{
	Init(); // Lambda initialization
	InitFromPatternData(Particles.Size(), bUse3dRestLengths, FaceVertexPatternPositions, TriangleMesh, SavePreResizedSimPosition3D, PreResizedSimPosition3D, MeshScale);
	InitColor(Particles);
}

FXPBDAnisotropicEdgeSpringConstraints::FXPBDAnisotropicEdgeSpringConstraints(
	const FSolverParticles& Particles,
	int32 InParticleOffset,
	int32 InParticleCount,
	const FTriangleMesh& TriangleMesh,
	const TArray<TVec3<FVec2f>>& FaceVertexPatternPositions,
	bool bUse3dRestLengths,
	bool SavePreResizedSimPosition3D,
	const TArray<FVector3f>& PreResizedSimPosition3D,
	FSolverReal MeshScale,
	const TConstArrayView<FRealSingle>& StiffnessWarpMultipliers,
	const TConstArrayView<FRealSingle>& StiffnessWeftMultipliers,
	const TConstArrayView<FRealSingle>& StiffnessBiasMultipliers,
	const TConstArrayView<FRealSingle>& DampingMultipliers,
	const TConstArrayView<FRealSingle>& WarpScaleMultipliers,
	const TConstArrayView<FRealSingle>& WeftScaleMultipliers,
	const FSolverVec2& InStiffnessWarp,
	const FSolverVec2& InStiffnessWeft,
	const FSolverVec2& InStiffnessBias,
	const FSolverVec2& InDampingRatio,
	const FSolverVec2& InWarpScale,
	const FSolverVec2& InWeftScale)
	: Base(Particles,
		InParticleOffset,
		InParticleCount,
		TriangleMesh.GetElements(),
		TConstArrayView<FRealSingle>(), // We don't use base stiffness maps
		InStiffnessWarp,
		true /*bTrimKinematicConstraints*/,
		MaxStiffness)
	, StiffnessWarp(InStiffnessWarp.ClampAxes(0, MaxStiffness),
		StiffnessWarpMultipliers,
		TConstArrayView<TVec2<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, StiffnessWeft(InStiffnessWeft.ClampAxes(0, MaxStiffness),
		StiffnessWeftMultipliers,
		TConstArrayView<TVec2<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, StiffnessBias(InStiffnessBias.ClampAxes(0, MaxStiffness),
		StiffnessBiasMultipliers,
		TConstArrayView<TVec2<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, DampingRatio(
		InDampingRatio.ClampAxes(MinDampingRatio, MaxDampingRatio),
		DampingMultipliers,
		TConstArrayView<TVec2<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, WarpScale(
		InWarpScale.ClampAxes(MinWarpWeftScale, MaxWarpWeftScale),
		WarpScaleMultipliers,
		TConstArrayView<TVec2<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, WeftScale(
		InWeftScale.ClampAxes(MinWarpWeftScale, MaxWarpWeftScale),
		WeftScaleMultipliers,
		TConstArrayView<TVec2<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
{
	Init(); // Lambda initialization
	InitFromPatternData(Particles.Size(), bUse3dRestLengths, FaceVertexPatternPositions, TriangleMesh, SavePreResizedSimPosition3D, PreResizedSimPosition3D, MeshScale);
	InitColor(Particles);
}

FXPBDAnisotropicEdgeSpringConstraints::FXPBDAnisotropicEdgeSpringConstraints(
	const FSolverParticlesRange& Particles,
	const FTriangleMesh& TriangleMesh,
	const TArray<TVec3<FVec2f>>& FaceVertexPatternPositions,
	bool bUse3dRestLengths,
	const TConstArrayView<FRealSingle>& StiffnessWarpMultipliers,
	const TConstArrayView<FRealSingle>& StiffnessWeftMultipliers,
	const TConstArrayView<FRealSingle>& StiffnessBiasMultipliers,
	const TConstArrayView<FRealSingle>& DampingMultipliers,
	const TConstArrayView<FRealSingle>& WarpScaleMultipliers,
	const TConstArrayView<FRealSingle>& WeftScaleMultipliers,
	const FSolverVec2& InStiffnessWarp,
	const FSolverVec2& InStiffnessWeft,
	const FSolverVec2& InStiffnessBias,
	const FSolverVec2& InDampingRatio,
	const FSolverVec2& InWarpScale,
	const FSolverVec2& InWeftScale)
	: Base(Particles,
		TriangleMesh.GetElements(),
		TConstArrayView<FRealSingle>(), // We don't use base stiffness maps
		InStiffnessWarp,
		true /*bTrimKinematicConstraints*/,
		MaxStiffness)
	, StiffnessWarp(InStiffnessWarp.ClampAxes(0, MaxStiffness),
		StiffnessWarpMultipliers,
		TConstArrayView<TVec2<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, StiffnessWeft(InStiffnessWeft.ClampAxes(0, MaxStiffness),
		StiffnessWeftMultipliers,
		TConstArrayView<TVec2<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, StiffnessBias(InStiffnessBias.ClampAxes(0, MaxStiffness),
		StiffnessBiasMultipliers,
		TConstArrayView<TVec2<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, DampingRatio(
		InDampingRatio.ClampAxes(MinDampingRatio, MaxDampingRatio),
		DampingMultipliers,
		TConstArrayView<TVec2<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, WarpScale(
		InWarpScale.ClampAxes(MinWarpWeftScale, MaxWarpWeftScale),
		WarpScaleMultipliers,
		TConstArrayView<TVec2<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, WeftScale(
		InWeftScale.ClampAxes(MinWarpWeftScale, MaxWarpWeftScale),
		WeftScaleMultipliers,
		TConstArrayView<TVec2<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
{
	Init(); // Lambda initialization
	InitFromPatternData(Particles.Size(), bUse3dRestLengths, FaceVertexPatternPositions, TriangleMesh, /*SavePreResizedSimPosition3D = */false, /*PreResizedSimPosition3D = */TArray<FVector3f>(), /*MeshScale = */ (FSolverReal)1.);
	InitColor(Particles);
}

FXPBDAnisotropicEdgeSpringConstraints::FXPBDAnisotropicEdgeSpringConstraints(
	const FSolverParticles& Particles,
	int32 InParticleOffset,
	int32 InParticleCount,
	const FTriangleMesh& TriangleMesh,
	const TArray<TVec3<FVec2f>>& FaceVertexPatternPositions,
	bool bUse3dRestLengths,
	const TConstArrayView<FRealSingle>& StiffnessWarpMultipliers,
	const TConstArrayView<FRealSingle>& StiffnessWeftMultipliers,
	const TConstArrayView<FRealSingle>& StiffnessBiasMultipliers,
	const TConstArrayView<FRealSingle>& DampingMultipliers,
	const TConstArrayView<FRealSingle>& WarpScaleMultipliers,
	const TConstArrayView<FRealSingle>& WeftScaleMultipliers,
	const FSolverVec2& InStiffnessWarp,
	const FSolverVec2& InStiffnessWeft,
	const FSolverVec2& InStiffnessBias,
	const FSolverVec2& InDampingRatio,
	const FSolverVec2& InWarpScale,
	const FSolverVec2& InWeftScale)
	: Base(Particles,
		InParticleOffset,
		InParticleCount,
		TriangleMesh.GetElements(),
		TConstArrayView<FRealSingle>(), // We don't use base stiffness maps
		InStiffnessWarp,
		true /*bTrimKinematicConstraints*/,
		MaxStiffness)
	, StiffnessWarp(InStiffnessWarp.ClampAxes(0, MaxStiffness),
		StiffnessWarpMultipliers,
		TConstArrayView<TVec2<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, StiffnessWeft(InStiffnessWeft.ClampAxes(0, MaxStiffness),
		StiffnessWeftMultipliers,
		TConstArrayView<TVec2<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, StiffnessBias(InStiffnessBias.ClampAxes(0, MaxStiffness),
		StiffnessBiasMultipliers,
		TConstArrayView<TVec2<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, DampingRatio(
		InDampingRatio.ClampAxes(MinDampingRatio, MaxDampingRatio),
		DampingMultipliers,
		TConstArrayView<TVec2<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, WarpScale(
		InWarpScale.ClampAxes(MinWarpWeftScale, MaxWarpWeftScale),
		WarpScaleMultipliers,
		TConstArrayView<TVec2<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, WeftScale(
		InWeftScale.ClampAxes(MinWarpWeftScale, MaxWarpWeftScale),
		WeftScaleMultipliers,
		TConstArrayView<TVec2<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
{
	Init(); // Lambda initialization
	InitFromPatternData(Particles.Size(), bUse3dRestLengths, FaceVertexPatternPositions, TriangleMesh, /*SavePreResizedSimPosition3D = */false, /*PreResizedSimPosition3D = */TArray<FVector3f>(), /*MeshScale = */ (FSolverReal)1.);
	InitColor(Particles);
}

void FXPBDAnisotropicEdgeSpringConstraints::InitFromPatternData(int32 ParticlesSize, bool bUse3dRestLengths, const TArray<TVec3<FVec2f>>& FaceVertexPatternPositions, const FTriangleMesh& TriangleMesh, bool SavePreResizedSimPosition3D, const TArray<FVector3f>& PreResizedSimPosition3D, FSolverReal MeshScale)
{
	// Calculate multipliers per face. Average the multipliers for faces that share the same edge.
	struct FMultipliers
	{
		FSolverVec3 WarpWeftBias;
		FSolverVec2 WarpWeftScale;
		FSolverReal Length;
	};
	TMap<TVec2<int32> /*Edge*/, TArray<FMultipliers>> EdgeToMultiplierMap;
	auto SortedEdge = [](int32 P0, int32 P1) { return P0 <= P1 ? TVec2<int32>(P0, P1) : TVec2<int32>(P1, P0); };
	auto Multiplier = [](const FVec2f& UV0, const FVec2f& UV1)
	{
		// Input UVs: X = Weft, Y = Warp direction
		// Transform to X = Weft, Y = Warp, Z = Bias for WarpWeftBias multiplier
		// 
		const FSolverVec2 UVDiff = UV1 - UV0;
		const FSolverVec2 UVDiffAbs = UVDiff.GetAbs();
		FSolverVec3 UVDiffTransformed;
		if (UVDiffAbs.X > UVDiffAbs.Y)
		{
			// Stiffness is blend between weft and bias
			UVDiffTransformed = FSolverVec3(UVDiffAbs.X - UVDiffAbs.Y, (FSolverReal)0.f, UVDiffAbs.Y);
		}
		else
		{
			// Stiffness is blend between warp and bias
			UVDiffTransformed = FSolverVec3((FSolverReal)0.f, UVDiffAbs.Y - UVDiffAbs.X, UVDiffAbs.X);
		}
		const FSolverReal Denom3 = UVDiffTransformed.X + UVDiffTransformed.Y + UVDiffTransformed.Z; // These are used as weights
		const FSolverReal UVLength = UVDiffAbs.Length();
		return FMultipliers
		{
			Denom3 > UE_SMALL_NUMBER ? UVDiffTransformed / Denom3 : FSolverVec3(0.f, 1.f, 0.f), // Default to Warp if zero length
			UVLength > UE_SMALL_NUMBER ? UVDiffAbs / UVLength : FSolverVec2(UE_INV_SQRT_2,UE_INV_SQRT_2), // Default to equally scaling warp and weft directions if zero length
			UVLength
		};
	};

	const TArray<TVec3<int32>>& Elements = TriangleMesh.GetElements();
	for (int32 ElemIdx = 0; ElemIdx < Elements.Num(); ++ElemIdx)
	{
		const TVec3<int32>& Element = Elements[ElemIdx];
		const TVec3<FVec2f>& UVs = FaceVertexPatternPositions[ElemIdx];
		EdgeToMultiplierMap.FindOrAdd(SortedEdge(Element[0], Element[1])).Add(Multiplier(UVs[0], UVs[1]));
		EdgeToMultiplierMap.FindOrAdd(SortedEdge(Element[1], Element[2])).Add(Multiplier(UVs[1], UVs[2]));
		EdgeToMultiplierMap.FindOrAdd(SortedEdge(Element[2], Element[0])).Add(Multiplier(UVs[2], UVs[0]));
	}

	if (bUse3dRestLengths)
	{
		BaseDists = Dists; // Use Dists calculated by base class using 3D positions
	}
	else
	{
		// We'll set them from the computed 2D lengths above
		BaseDists.SetNumZeroed(Constraints.Num());
	}
	WarpWeftBiasBaseMultipliers.SetNumZeroed(Constraints.Num());
	WarpWeftScaleBaseMultipliers.SetNumZeroed(Constraints.Num());
	for (int32 ConstraintIdx = 0; ConstraintIdx < Constraints.Num(); ++ConstraintIdx)
	{
		const TArray<FMultipliers>& EdgeMultipliers = EdgeToMultiplierMap.FindChecked(SortedEdge(Constraints[ConstraintIdx][0], Constraints[ConstraintIdx][1]));
		check(EdgeMultipliers.Num());
		for (const FMultipliers& EdgeMult : EdgeMultipliers)
		{
			WarpWeftBiasBaseMultipliers[ConstraintIdx] += EdgeMult.WarpWeftBias;
			WarpWeftScaleBaseMultipliers[ConstraintIdx] += EdgeMult.WarpWeftScale;
			if (!bUse3dRestLengths)
			{
				BaseDists[ConstraintIdx] += EdgeMult.Length;
			}
		}
		WarpWeftBiasBaseMultipliers[ConstraintIdx] /= (FSolverReal)EdgeMultipliers.Num();
		WarpWeftScaleBaseMultipliers[ConstraintIdx].Normalize();
		if (!bUse3dRestLengths)
		{
			if (SavePreResizedSimPosition3D && (ParticlesSize == PreResizedSimPosition3D.Num()))
			{
				const FSolverReal Denom = MeshScale * (PreResizedSimPosition3D[Constraints[ConstraintIdx][0]] - PreResizedSimPosition3D[Constraints[ConstraintIdx][1]]).Size();
				const FSolverReal Scale = Denom > UE_SMALL_NUMBER ? Dists[ConstraintIdx] / Denom : 1;
				BaseDists[ConstraintIdx] *= Scale;
			}
			BaseDists[ConstraintIdx] /= (FSolverReal)EdgeMultipliers.Num();
			Dists[ConstraintIdx] = BaseDists[ConstraintIdx];
		}
	}
}

template<typename SolverParticlesOrRange>
void FXPBDAnisotropicEdgeSpringConstraints::InitColor(const SolverParticlesOrRange& InParticles)
{
	// In dev builds we always color so we can tune the system without restarting. See Apply()
#if UE_BUILD_SHIPPING || UE_BUILD_TEST
	if (Constraints.Num() > Chaos_XPBDSpring_ParallelConstraintCount)
#endif
	{
		const TArray<TArray<int32>> ConstraintsPerColor = FGraphColoring::ComputeGraphColoringParticlesOrRange(Constraints, InParticles, ParticleOffset, ParticleOffset + ParticleCount);

		// Reorder constraints based on color so each array in ConstraintsPerColor contains contiguous elements.
		TArray<TVec2<int32>> ReorderedConstraints;
		TArray<FSolverReal> ReorderedDists;
		TArray<FSolverReal> ReorderedBaseDists;
		TArray<FSolverVec3> ReorderedWarpWeftBiasBaseMultipliers;
		TArray<FSolverVec2> ReorderedWarpWeftScaleBaseMultipliers;
		TArray<int32> OrigToReorderedIndices; // used to reorder stiffness indices
		ReorderedConstraints.SetNumUninitialized(Constraints.Num());
		ReorderedDists.SetNumUninitialized(Constraints.Num());
		ReorderedBaseDists.SetNumUninitialized(Constraints.Num());
		ReorderedWarpWeftBiasBaseMultipliers.SetNumUninitialized(Constraints.Num());
		ReorderedWarpWeftScaleBaseMultipliers.SetNumUninitialized(Constraints.Num());
		OrigToReorderedIndices.SetNumUninitialized(Constraints.Num());

		ConstraintsPerColorStartIndex.Reset(ConstraintsPerColor.Num() + 1);

		int32 ReorderedIndex = 0;
		for (const TArray<int32>& ConstraintsBatch : ConstraintsPerColor)
		{
			ConstraintsPerColorStartIndex.Add(ReorderedIndex);
			for (const int32& BatchConstraint : ConstraintsBatch)
			{
				const int32 OrigIndex = BatchConstraint;
				ReorderedConstraints[ReorderedIndex] = Constraints[OrigIndex];
				ReorderedDists[ReorderedIndex] = Dists[OrigIndex];
				ReorderedBaseDists[ReorderedIndex] = BaseDists[OrigIndex];
				ReorderedWarpWeftBiasBaseMultipliers[ReorderedIndex] = WarpWeftBiasBaseMultipliers[OrigIndex];
				ReorderedWarpWeftScaleBaseMultipliers[ReorderedIndex] = WarpWeftScaleBaseMultipliers[OrigIndex];
				OrigToReorderedIndices[OrigIndex] = ReorderedIndex;
				++ReorderedIndex;
			}
		}
		ConstraintsPerColorStartIndex.Add(ReorderedIndex);

		Constraints = MoveTemp(ReorderedConstraints);
		Dists = MoveTemp(ReorderedDists);
		BaseDists = MoveTemp(ReorderedBaseDists);
		WarpWeftBiasBaseMultipliers = MoveTemp(ReorderedWarpWeftBiasBaseMultipliers);
		WarpWeftScaleBaseMultipliers = MoveTemp(ReorderedWarpWeftScaleBaseMultipliers);
		Stiffness.ReorderIndices(OrigToReorderedIndices);
		StiffnessWeft.ReorderIndices(OrigToReorderedIndices);
		StiffnessBias.ReorderIndices(OrigToReorderedIndices);
		DampingRatio.ReorderIndices(OrigToReorderedIndices);
		WarpScale.ReorderIndices(OrigToReorderedIndices);
		WeftScale.ReorderIndices(OrigToReorderedIndices);
	}
}

void FXPBDAnisotropicEdgeSpringConstraints::UpdateDists()
{
	Private::UpdateDists(BaseDists, WarpWeftScaleBaseMultipliers, WarpScale, WeftScale, Dists);
}

template<bool bDampingBefore, bool bSingleLambda, bool bSeparateStretch, bool bDampingAfter, typename SolverParticlesOrRange>
void FXPBDAnisotropicEdgeSpringConstraints::ApplyHelper(SolverParticlesOrRange& Particles, const FSolverReal Dt, const int32 ConstraintIndex, const FSolverVec3& ExpStiffnessValues, const FSolverReal DampingRatioValue) const
{
	const TVec2<int32>& Constraint = Constraints[ConstraintIndex];
	const int32 Index1 = Constraint[0];
	const int32 Index2 = Constraint[1];

	const FSolverVec3& WarpWeftBiasBaseMultiplier = WarpWeftBiasBaseMultipliers[ConstraintIndex];
	const FSolverReal ExpStiffnessValue = WarpWeftBiasBaseMultiplier.Dot(ExpStiffnessValues);

	FSolverVec3 Delta(0.f);
	if constexpr (bDampingBefore)
	{
		Delta += Spring::GetXPBDSpringDampingDelta(Particles, Dt, Constraint, Dists[ConstraintIndex], LambdasDamping[ConstraintIndex],
			ExpStiffnessValue, DampingRatioValue);
	}

	if constexpr (bSingleLambda)
	{
		Delta += Spring::GetXPBDSpringDeltaWithDamping(Particles, Dt, Constraint, Dists[ConstraintIndex], Lambdas[ConstraintIndex],
			ExpStiffnessValue, DampingRatioValue);
	}

	if constexpr (bSeparateStretch)
	{
		Delta += Spring::GetXPBDSpringDelta(Particles, Dt, Constraint, Dists[ConstraintIndex], Lambdas[ConstraintIndex],
			ExpStiffnessValue);
	}

	if constexpr (bDampingAfter)
	{
		Delta += Spring::GetXPBDSpringDampingDelta(Particles, Dt, Constraint, Dists[ConstraintIndex], LambdasDamping[ConstraintIndex],
			ExpStiffnessValue, DampingRatioValue);
	}

	Particles.P(Index1) += Particles.InvM(Index1) * Delta;
	Particles.P(Index2) -= Particles.InvM(Index2) * Delta;
}

template<typename SolverParticlesOrRange>
void FXPBDAnisotropicEdgeSpringConstraints::Apply(SolverParticlesOrRange& Particles, const FSolverReal Dt) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FXPBDAnisotropicEdgeSpringConstraints_Apply);
	SCOPE_CYCLE_COUNTER(STAT_XPBD_AnisoSpring);

	const bool StiffnessHasWeightMap = StiffnessWarp.HasWeightMap();
	const bool StiffnessWeftHasWeightMap = StiffnessWeft.HasWeightMap();
	const bool StiffnessBiasHasWeightMap = StiffnessBias.HasWeightMap();
	const bool DampingHasWeightMap = DampingRatio.HasWeightMap();

	if ((ConstraintsPerColorStartIndex.Num() > 1) && (Constraints.Num() > Chaos_XPBDSpring_ParallelConstraintCount))
	{
		const int32 ConstraintColorNum = ConstraintsPerColorStartIndex.Num() - 1;
#if INTEL_ISPC
		if (bRealTypeCompatibleWithISPC && bChaos_XPBDSpring_ISPC_Enabled)
		{
			const bool bSingleLambda = Chaos_XPBDSpring_SplitDampingMode == (int32)EXPBDSplitDampingMode::SingleLambda;

			if (!StiffnessHasWeightMap && !StiffnessWeftHasWeightMap && !StiffnessBiasHasWeightMap && !DampingHasWeightMap)
			{
				const FSolverVec3 ExpStiffnessValue((FSolverReal)StiffnessWeft, (FSolverReal)StiffnessWarp, (FSolverReal)StiffnessBias);
				const FSolverReal DampingRatioValue = (FSolverReal)DampingRatio;
				if (ExpStiffnessValue.Max() <= MinStiffness)
				{
					return;
				}

				if (DampingRatioValue > 0)
				{
					if (bSingleLambda)
					{
						for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
						{
							const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
							const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
							ispc::ApplyXPBDAnisoSpringConstraintsWithDamping(
								(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
								(const ispc::FVector3f*)Particles.XArray().GetData(),
								(ispc::FIntVector2*)&Constraints.GetData()[ColorStart],
								&Dists.GetData()[ColorStart],
								(const ispc::FVector3f*)&WarpWeftBiasBaseMultipliers.GetData()[ColorStart],
								&Lambdas.GetData()[ColorStart],
								Dt,
								reinterpret_cast<const ispc::FVector3f&>(ExpStiffnessValue),
								DampingRatioValue,
								ColorSize);
						}
						return;
					}
					else
					{
						for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
						{
							const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
							const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
							ispc::ApplyXPBDAnisoSpringDampingConstraints(
								(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
								(const ispc::FVector3f*)Particles.XArray().GetData(),
								(ispc::FIntVector2*)&Constraints.GetData()[ColorStart],
								&Dists.GetData()[ColorStart],
								(const ispc::FVector3f*)&WarpWeftBiasBaseMultipliers.GetData()[ColorStart],
								&LambdasDamping.GetData()[ColorStart],
								Dt,
								reinterpret_cast<const ispc::FVector3f&>(ExpStiffnessValue),
								DampingRatioValue,
								ColorSize);
						}
					}
				}
				for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
				{
					const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
					const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
					ispc::ApplyXPBDAnisoSpringConstraints(
						(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
						(ispc::FIntVector2*)&Constraints.GetData()[ColorStart],
						&Dists.GetData()[ColorStart],
						(const ispc::FVector3f*)&WarpWeftBiasBaseMultipliers.GetData()[ColorStart],
						&Lambdas.GetData()[ColorStart],
						Dt,
						reinterpret_cast<const ispc::FVector3f&>(ExpStiffnessValue),
						ColorSize);
				}
			}
			else // ISPC with maps
			{
				if (DampingHasWeightMap || (FSolverReal)DampingRatio > 0)
				{
					if (bSingleLambda)
					{
						for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
						{
							const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
							const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
							ispc::ApplyXPBDAnisoSpringConstraintsWithDampingAndWeightMaps(
								(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
								(const ispc::FVector3f*)Particles.XArray().GetData(),
								(ispc::FIntVector2*)&Constraints.GetData()[ColorStart],
								&Dists.GetData()[ColorStart],
								(const ispc::FVector3f*)&WarpWeftBiasBaseMultipliers.GetData()[ColorStart],
								&Lambdas.GetData()[ColorStart],
								Dt,
								StiffnessHasWeightMap,
								reinterpret_cast<const ispc::FVector2f&>(StiffnessWarp.GetOffsetRange()),
								StiffnessHasWeightMap ? &StiffnessWarp.GetMapValues().GetData()[ColorStart] : nullptr,
								StiffnessWeftHasWeightMap,
								reinterpret_cast<const ispc::FVector2f&>(StiffnessWeft.GetOffsetRange()),
								StiffnessWeftHasWeightMap ? &StiffnessWeft.GetMapValues().GetData()[ColorStart] : nullptr,
								StiffnessBiasHasWeightMap,
								reinterpret_cast<const ispc::FVector2f&>(StiffnessBias.GetOffsetRange()),
								StiffnessBiasHasWeightMap ? &StiffnessBias.GetMapValues().GetData()[ColorStart] : nullptr,
								DampingHasWeightMap,
								reinterpret_cast<const ispc::FVector2f&>(DampingRatio.GetOffsetRange()),
								DampingHasWeightMap ? &DampingRatio.GetMapValues()[ColorStart] : nullptr,
								ColorSize);
						}
						return;
					}
					else
					{
						for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
						{
							const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
							const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
							ispc::ApplyXPBDAnisoSpringDampingConstraintsWithWeightMaps(
								(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
								(const ispc::FVector3f*)Particles.XArray().GetData(),
								(ispc::FIntVector2*)&Constraints.GetData()[ColorStart],
								&Dists.GetData()[ColorStart],
								(const ispc::FVector3f*)&WarpWeftBiasBaseMultipliers.GetData()[ColorStart],
								&LambdasDamping.GetData()[ColorStart],
								Dt,
								StiffnessHasWeightMap,
								reinterpret_cast<const ispc::FVector2f&>(StiffnessWarp.GetOffsetRange()),
								StiffnessHasWeightMap ? &StiffnessWarp.GetMapValues().GetData()[ColorStart] : nullptr,
								StiffnessWeftHasWeightMap,
								reinterpret_cast<const ispc::FVector2f&>(StiffnessWeft.GetOffsetRange()),
								StiffnessWeftHasWeightMap ? &StiffnessWeft.GetMapValues().GetData()[ColorStart] : nullptr,
								StiffnessBiasHasWeightMap,
								reinterpret_cast<const ispc::FVector2f&>(StiffnessBias.GetOffsetRange()),
								StiffnessBiasHasWeightMap ? &StiffnessBias.GetMapValues().GetData()[ColorStart] : nullptr,
								DampingHasWeightMap,
								reinterpret_cast<const ispc::FVector2f&>(DampingRatio.GetOffsetRange()),
								DampingHasWeightMap ? &DampingRatio.GetMapValues()[ColorStart] : nullptr,
								ColorSize);
						}
					}
				}
				for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
				{
					const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
					const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
					ispc::ApplyXPBDAnisoSpringConstraintsWithWeightMaps(
						(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
						(ispc::FIntVector2*)&Constraints.GetData()[ColorStart],
						&Dists.GetData()[ColorStart],
						(const ispc::FVector3f*)&WarpWeftBiasBaseMultipliers.GetData()[ColorStart],
						&Lambdas.GetData()[ColorStart],
						Dt,
						StiffnessHasWeightMap,
						reinterpret_cast<const ispc::FVector2f&>(StiffnessWarp.GetOffsetRange()),
						StiffnessHasWeightMap ? &StiffnessWarp.GetMapValues().GetData()[ColorStart] : nullptr,
						StiffnessWeftHasWeightMap,
						reinterpret_cast<const ispc::FVector2f&>(StiffnessWeft.GetOffsetRange()),
						StiffnessWeftHasWeightMap ? &StiffnessWeft.GetMapValues().GetData()[ColorStart] : nullptr,
						StiffnessBiasHasWeightMap,
						reinterpret_cast<const ispc::FVector2f&>(StiffnessBias.GetOffsetRange()),
						StiffnessBiasHasWeightMap ? &StiffnessBias.GetMapValues().GetData()[ColorStart] : nullptr,
						ColorSize);
				}
			}
		}
		else
#endif
		{
			// Parallel non-ispc
			const FSolverReal StiffnessNoMap = (FSolverReal)StiffnessWarp;
			const FSolverReal StiffnessWeftNoMap = (FSolverReal)StiffnessWeft;
			const FSolverReal StiffnessBiasNoMap = (FSolverReal)StiffnessBias;
			const FSolverReal DampingNoMap = (FSolverReal)DampingRatio;
			if (DampingNoMap > 0 || DampingHasWeightMap)
			{
				if (Chaos_XPBDSpring_SplitDampingMode == (int32)EXPBDSplitDampingMode::TwoPassBefore)
				{
					for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
					{
						const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
						const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
						PhysicsParallelFor(ColorSize, [this, &Particles, Dt, ColorStart, StiffnessHasWeightMap, StiffnessNoMap, StiffnessWeftHasWeightMap, StiffnessWeftNoMap, StiffnessBiasHasWeightMap, StiffnessBiasNoMap, DampingHasWeightMap, DampingNoMap](const int32 Index)
						{
							const int32 ConstraintIndex = ColorStart + Index;
							constexpr bool bDampingBefore = true;
							constexpr bool bSingleLambda = false;
							constexpr bool bSeparateStretch = false;
							constexpr bool bDampingAfter = false;
							const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? StiffnessWarp[ConstraintIndex] : StiffnessNoMap;
							const FSolverReal ExpStiffnessWeftValue = StiffnessWeftHasWeightMap ? StiffnessWeft[ConstraintIndex] : StiffnessWeftNoMap;
							const FSolverReal ExpStiffnessBiasValue = StiffnessBiasHasWeightMap ? StiffnessBias[ConstraintIndex] : StiffnessBiasNoMap;
							const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
							ApplyHelper<bDampingBefore, bSingleLambda, bSeparateStretch, bDampingAfter>(Particles, Dt, ConstraintIndex, FSolverVec3(ExpStiffnessWeftValue, ExpStiffnessValue, ExpStiffnessBiasValue), DampingRatioValue);
						});
					}
				}
				
				// Stretch part (possibly with damping depending on split mode)
				switch (Chaos_XPBDSpring_SplitDampingMode)
				{
				case (int32)EXPBDSplitDampingMode::TwoPassBefore: // fallthrough
				case (int32)EXPBDSplitDampingMode::TwoPassAfter: // fallthrough
				default:
				{
					// Do a pass with stretch only
					for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
					{
						const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
						const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
						PhysicsParallelFor(ColorSize, [this, &Particles, Dt, ColorStart, StiffnessHasWeightMap, StiffnessNoMap, StiffnessWeftHasWeightMap, StiffnessWeftNoMap, StiffnessBiasHasWeightMap, StiffnessBiasNoMap, DampingHasWeightMap, DampingNoMap](const int32 Index)
						{
							const int32 ConstraintIndex = ColorStart + Index;
							constexpr bool bDampingBefore = false;
							constexpr bool bSingleLambda = false;
							constexpr bool bSeparateStretch = true;
							constexpr bool bDampingAfter = false;
							const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? StiffnessWarp[ConstraintIndex] : StiffnessNoMap;
							const FSolverReal ExpStiffnessWeftValue = StiffnessWeftHasWeightMap ? StiffnessWeft[ConstraintIndex] : StiffnessWeftNoMap;
							const FSolverReal ExpStiffnessBiasValue = StiffnessBiasHasWeightMap ? StiffnessBias[ConstraintIndex] : StiffnessBiasNoMap;
							const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
							ApplyHelper<bDampingBefore, bSingleLambda, bSeparateStretch, bDampingAfter>(Particles, Dt, ConstraintIndex, FSolverVec3(ExpStiffnessWeftValue, ExpStiffnessValue, ExpStiffnessBiasValue), DampingRatioValue);
						});
					}
				}
				break;
				case (int32)EXPBDSplitDampingMode::SingleLambda:
				{
					// Do a pass with combined stretch and damping with a single lambda
					for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
					{
						const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
						const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
						PhysicsParallelFor(ColorSize, [this, &Particles, Dt, ColorStart, StiffnessHasWeightMap, StiffnessNoMap, StiffnessWeftHasWeightMap, StiffnessWeftNoMap, StiffnessBiasHasWeightMap, StiffnessBiasNoMap, DampingHasWeightMap, DampingNoMap](const int32 Index)
						{
							const int32 ConstraintIndex = ColorStart + Index;
							constexpr bool bDampingBefore = false;
							constexpr bool bSingleLambda = true;
							constexpr bool bSeparateStretch = false;
							constexpr bool bDampingAfter = false;
							const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? StiffnessWarp[ConstraintIndex] : StiffnessNoMap;
							const FSolverReal ExpStiffnessWeftValue = StiffnessWeftHasWeightMap ? StiffnessWeft[ConstraintIndex] : StiffnessWeftNoMap;
							const FSolverReal ExpStiffnessBiasValue = StiffnessBiasHasWeightMap ? StiffnessBias[ConstraintIndex] : StiffnessBiasNoMap;
							const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
							ApplyHelper<bDampingBefore, bSingleLambda, bSeparateStretch, bDampingAfter>(Particles, Dt, ConstraintIndex, FSolverVec3(ExpStiffnessWeftValue, ExpStiffnessValue, ExpStiffnessBiasValue), DampingRatioValue);
						});
					}
				}
				break;
				case (int32)EXPBDSplitDampingMode::InterleavedBefore:
				{
					// Do a pass with damping before stretch
					for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
					{
						const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
						const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
						PhysicsParallelFor(ColorSize, [this, &Particles, Dt, ColorStart, StiffnessHasWeightMap, StiffnessNoMap, StiffnessWeftHasWeightMap, StiffnessWeftNoMap, StiffnessBiasHasWeightMap, StiffnessBiasNoMap, DampingHasWeightMap, DampingNoMap](const int32 Index)
						{
							const int32 ConstraintIndex = ColorStart + Index;
							constexpr bool bDampingBefore = true;
							constexpr bool bSingleLambda = false;
							constexpr bool bSeparateStretch = true;
							constexpr bool bDampingAfter = false;
							const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? StiffnessWarp[ConstraintIndex] : StiffnessNoMap;
							const FSolverReal ExpStiffnessWeftValue = StiffnessWeftHasWeightMap ? StiffnessWeft[ConstraintIndex] : StiffnessWeftNoMap;
							const FSolverReal ExpStiffnessBiasValue = StiffnessBiasHasWeightMap ? StiffnessBias[ConstraintIndex] : StiffnessBiasNoMap;
							const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
							ApplyHelper<bDampingBefore, bSingleLambda, bSeparateStretch, bDampingAfter>(Particles, Dt, ConstraintIndex, FSolverVec3(ExpStiffnessWeftValue, ExpStiffnessValue, ExpStiffnessBiasValue), DampingRatioValue);
						});
					}
				}
				break;
				case (int32)EXPBDSplitDampingMode::InterleavedAfter:
				{
					// Do a pass with damping after stretch
					for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
					{
						const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
						const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
						PhysicsParallelFor(ColorSize, [this, &Particles, Dt, ColorStart, StiffnessHasWeightMap, StiffnessNoMap, StiffnessWeftHasWeightMap, StiffnessWeftNoMap, StiffnessBiasHasWeightMap, StiffnessBiasNoMap, DampingHasWeightMap, DampingNoMap](const int32 Index)
						{
							const int32 ConstraintIndex = ColorStart + Index;
							constexpr bool bDampingBefore = false;
							constexpr bool bSingleLambda = false;
							constexpr bool bSeparateStretch = true;
							constexpr bool bDampingAfter = true;
							const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? StiffnessWarp[ConstraintIndex] : StiffnessNoMap;
							const FSolverReal ExpStiffnessWeftValue = StiffnessWeftHasWeightMap ? StiffnessWeft[ConstraintIndex] : StiffnessWeftNoMap;
							const FSolverReal ExpStiffnessBiasValue = StiffnessBiasHasWeightMap ? StiffnessBias[ConstraintIndex] : StiffnessBiasNoMap;
							const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
							ApplyHelper<bDampingBefore, bSingleLambda, bSeparateStretch, bDampingAfter>(Particles, Dt, ConstraintIndex, FSolverVec3(ExpStiffnessWeftValue, ExpStiffnessValue, ExpStiffnessBiasValue), DampingRatioValue);
						});
					}
				}
				break;
				}
				if (Chaos_XPBDSpring_SplitDampingMode == (int32)EXPBDSplitDampingMode::TwoPassAfter)
				{
					for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
					{
						const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
						const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
						PhysicsParallelFor(ColorSize, [this, &Particles, Dt, ColorStart, StiffnessHasWeightMap, StiffnessNoMap, StiffnessWeftHasWeightMap, StiffnessWeftNoMap, StiffnessBiasHasWeightMap, StiffnessBiasNoMap, DampingHasWeightMap, DampingNoMap](const int32 Index)
						{
							const int32 ConstraintIndex = ColorStart + Index;
							constexpr bool bDampingBefore = false;
							constexpr bool bSingleLambda = false;
							constexpr bool bSeparateStretch = false;
							constexpr bool bDampingAfter = true;
							const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? StiffnessWarp[ConstraintIndex] : StiffnessNoMap;
							const FSolverReal ExpStiffnessWeftValue = StiffnessWeftHasWeightMap ? StiffnessWeft[ConstraintIndex] : StiffnessWeftNoMap;
							const FSolverReal ExpStiffnessBiasValue = StiffnessBiasHasWeightMap ? StiffnessBias[ConstraintIndex] : StiffnessBiasNoMap;
							const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
							ApplyHelper<bDampingBefore, bSingleLambda, bSeparateStretch, bDampingAfter>(Particles, Dt, ConstraintIndex, FSolverVec3(ExpStiffnessWeftValue, ExpStiffnessValue, ExpStiffnessBiasValue), DampingRatioValue);
						});
					}
				}
			}
			else
			{
				// No damping. 
				for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
				{
					const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
					const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
					PhysicsParallelFor(ColorSize, [this, &Particles, Dt, ColorStart, StiffnessHasWeightMap, StiffnessNoMap, StiffnessWeftHasWeightMap, StiffnessWeftNoMap, StiffnessBiasHasWeightMap, StiffnessBiasNoMap](const int32 Index)
					{
						const int32 ConstraintIndex = ColorStart + Index;
						constexpr bool bDampingBefore = false;
						constexpr bool bSingleLambda = false;
						constexpr bool bSeparateStretch = true;
						constexpr bool bDampingAfter = false;
						const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? StiffnessWarp[ConstraintIndex] : StiffnessNoMap;
						const FSolverReal ExpStiffnessWeftValue = StiffnessWeftHasWeightMap ? StiffnessWeft[ConstraintIndex] : StiffnessWeftNoMap;
						const FSolverReal ExpStiffnessBiasValue = StiffnessBiasHasWeightMap ? StiffnessBias[ConstraintIndex] : StiffnessBiasNoMap;
						ApplyHelper<bDampingBefore, bSingleLambda, bSeparateStretch, bDampingAfter>(Particles, Dt, ConstraintIndex, FSolverVec3(ExpStiffnessWeftValue, ExpStiffnessValue, ExpStiffnessBiasValue), (FSolverReal)0.);
					});
				}
			}
		}
	}
	else
	{
		// Single threaded
		const FSolverReal StiffnessNoMap = (FSolverReal)StiffnessWarp;
		const FSolverReal StiffnessWeftNoMap = (FSolverReal)StiffnessWeft;
		const FSolverReal StiffnessBiasNoMap = (FSolverReal)StiffnessBias;
		const FSolverReal DampingNoMap = (FSolverReal)DampingRatio;
		if (DampingNoMap > 0 || DampingHasWeightMap)
		{
			if (Chaos_XPBDSpring_SplitDampingMode == (int32)EXPBDSplitDampingMode::TwoPassBefore)
			{
				for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
				{
					constexpr bool bDampingBefore = true;
					constexpr bool bSingleLambda = false;
					constexpr bool bSeparateStretch = false;
					constexpr bool bDampingAfter = false;
					const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? StiffnessWarp[ConstraintIndex] : StiffnessNoMap;
					const FSolverReal ExpStiffnessWeftValue = StiffnessWeftHasWeightMap ? StiffnessWeft[ConstraintIndex] : StiffnessWeftNoMap;
					const FSolverReal ExpStiffnessBiasValue = StiffnessBiasHasWeightMap ? StiffnessBias[ConstraintIndex] : StiffnessBiasNoMap;
					const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
					ApplyHelper<bDampingBefore, bSingleLambda, bSeparateStretch, bDampingAfter>(Particles, Dt, ConstraintIndex, FSolverVec3(ExpStiffnessWeftValue, ExpStiffnessValue, ExpStiffnessBiasValue), DampingRatioValue);
				}
			}

			// Stretch part (possibly with damping depending on split mode)
			switch (Chaos_XPBDSpring_SplitDampingMode)
			{
			case (int32)EXPBDSplitDampingMode::TwoPassBefore: // fallthrough
			case (int32)EXPBDSplitDampingMode::TwoPassAfter: // fallthrough
			default:
			{
				// Do a pass with stretch only
				for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
				{
					constexpr bool bDampingBefore = false;
					constexpr bool bSingleLambda = false;
					constexpr bool bSeparateStretch = true;
					constexpr bool bDampingAfter = false;
					const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? StiffnessWarp[ConstraintIndex] : StiffnessNoMap;
					const FSolverReal ExpStiffnessWeftValue = StiffnessWeftHasWeightMap ? StiffnessWeft[ConstraintIndex] : StiffnessWeftNoMap;
					const FSolverReal ExpStiffnessBiasValue = StiffnessBiasHasWeightMap ? StiffnessBias[ConstraintIndex] : StiffnessBiasNoMap;
					const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
					ApplyHelper<bDampingBefore, bSingleLambda, bSeparateStretch, bDampingAfter>(Particles, Dt, ConstraintIndex, FSolverVec3(ExpStiffnessWeftValue, ExpStiffnessValue, ExpStiffnessBiasValue), DampingRatioValue);
				}
			}
			break;
			case (int32)EXPBDSplitDampingMode::SingleLambda:
			{
				// Do a pass with combined stretch and damping with a single lambda
				for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
				{
					constexpr bool bDampingBefore = false;
					constexpr bool bSingleLambda = true;
					constexpr bool bSeparateStretch = false;
					constexpr bool bDampingAfter = false;
					const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? StiffnessWarp[ConstraintIndex] : StiffnessNoMap;
					const FSolverReal ExpStiffnessWeftValue = StiffnessWeftHasWeightMap ? StiffnessWeft[ConstraintIndex] : StiffnessWeftNoMap;
					const FSolverReal ExpStiffnessBiasValue = StiffnessBiasHasWeightMap ? StiffnessBias[ConstraintIndex] : StiffnessBiasNoMap;
					const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
					ApplyHelper<bDampingBefore, bSingleLambda, bSeparateStretch, bDampingAfter>(Particles, Dt, ConstraintIndex, FSolverVec3(ExpStiffnessWeftValue, ExpStiffnessValue, ExpStiffnessBiasValue), DampingRatioValue);
				}
			}
			break;
			case (int32)EXPBDSplitDampingMode::InterleavedBefore:
			{
				// Do a pass with damping before stretch
				for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
				{
					constexpr bool bDampingBefore = true;
					constexpr bool bSingleLambda = false;
					constexpr bool bSeparateStretch = true;
					constexpr bool bDampingAfter = false;
					const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? StiffnessWarp[ConstraintIndex] : StiffnessNoMap;
					const FSolverReal ExpStiffnessWeftValue = StiffnessWeftHasWeightMap ? StiffnessWeft[ConstraintIndex] : StiffnessWeftNoMap;
					const FSolverReal ExpStiffnessBiasValue = StiffnessBiasHasWeightMap ? StiffnessBias[ConstraintIndex] : StiffnessBiasNoMap;
					const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
					ApplyHelper<bDampingBefore, bSingleLambda, bSeparateStretch, bDampingAfter>(Particles, Dt, ConstraintIndex, FSolverVec3(ExpStiffnessWeftValue, ExpStiffnessValue, ExpStiffnessBiasValue), DampingRatioValue);
				}
			}
			break;
			case (int32)EXPBDSplitDampingMode::InterleavedAfter:
			{
				// Do a pass with damping after stretch
				for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
				{
					constexpr bool bDampingBefore = false;
					constexpr bool bSingleLambda = false;
					constexpr bool bSeparateStretch = true;
					constexpr bool bDampingAfter = true;
					const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? StiffnessWarp[ConstraintIndex] : StiffnessNoMap;
					const FSolverReal ExpStiffnessWeftValue = StiffnessWeftHasWeightMap ? StiffnessWeft[ConstraintIndex] : StiffnessWeftNoMap;
					const FSolverReal ExpStiffnessBiasValue = StiffnessBiasHasWeightMap ? StiffnessBias[ConstraintIndex] : StiffnessBiasNoMap;
					const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
					ApplyHelper<bDampingBefore, bSingleLambda, bSeparateStretch, bDampingAfter>(Particles, Dt, ConstraintIndex, FSolverVec3(ExpStiffnessWeftValue, ExpStiffnessValue, ExpStiffnessBiasValue), DampingRatioValue);
				}
			}
			break;
			}
			if (Chaos_XPBDSpring_SplitDampingMode == (int32)EXPBDSplitDampingMode::TwoPassAfter)
			{
				for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
				{
					constexpr bool bDampingBefore = false;
					constexpr bool bSingleLambda = false;
					constexpr bool bSeparateStretch = false;
					constexpr bool bDampingAfter = true;
					const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? StiffnessWarp[ConstraintIndex] : StiffnessNoMap;
					const FSolverReal ExpStiffnessWeftValue = StiffnessWeftHasWeightMap ? StiffnessWeft[ConstraintIndex] : StiffnessWeftNoMap;
					const FSolverReal ExpStiffnessBiasValue = StiffnessBiasHasWeightMap ? StiffnessBias[ConstraintIndex] : StiffnessBiasNoMap;
					const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
					ApplyHelper<bDampingBefore, bSingleLambda, bSeparateStretch, bDampingAfter>(Particles, Dt, ConstraintIndex, FSolverVec3(ExpStiffnessWeftValue, ExpStiffnessValue, ExpStiffnessBiasValue), DampingRatioValue);
				}
			}
		}
		else
		{
			// No damping. 
			for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
			{
				constexpr bool bDampingBefore = false;
				constexpr bool bSingleLambda = false;
				constexpr bool bSeparateStretch = true;
				constexpr bool bDampingAfter = false;
				const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? StiffnessWarp[ConstraintIndex] : StiffnessNoMap;
				const FSolverReal ExpStiffnessWeftValue = StiffnessWeftHasWeightMap ? StiffnessWeft[ConstraintIndex] : StiffnessWeftNoMap;
				const FSolverReal ExpStiffnessBiasValue = StiffnessBiasHasWeightMap ? StiffnessBias[ConstraintIndex] : StiffnessBiasNoMap;
				ApplyHelper<bDampingBefore, bSingleLambda, bSeparateStretch, bDampingAfter>(Particles, Dt, ConstraintIndex, FSolverVec3(ExpStiffnessWeftValue, ExpStiffnessValue, ExpStiffnessBiasValue), (FSolverReal)0.);

			}
		}
	}
}
template CHAOS_API void FXPBDAnisotropicEdgeSpringConstraints::Apply(FSolverParticles& Particles, const FSolverReal Dt) const;
template CHAOS_API void FXPBDAnisotropicEdgeSpringConstraints::Apply(FSolverParticlesRange& Particles, const FSolverReal Dt) const;

void FXPBDAnisotropicEdgeSpringConstraints::UpdateLinearSystem(const FSolverParticlesRange& Particles, const FSolverReal Dt, FEvolutionLinearSystem& LinearSystem) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FXPBDAnisotropicEdgeSpringConstraints_UpdateLinearSystem);
	LinearSystem.ReserveForParallelAdd(Constraints.Num() * 2, Constraints.Num());

	const bool StiffnessHasWeightMap = StiffnessWarp.HasWeightMap();
	const bool StiffnessWeftHasWeightMap = StiffnessWeft.HasWeightMap();
	const bool StiffnessBiasHasWeightMap = StiffnessBias.HasWeightMap();
	const bool DampingHasWeightMap = DampingRatio.HasWeightMap();
	if ((ConstraintsPerColorStartIndex.Num() > 1) && (Constraints.Num() > Chaos_XPBDSpring_ParallelConstraintCount))
	{
		const int32 ConstraintColorNum = ConstraintsPerColorStartIndex.Num() - 1;
		if (!StiffnessHasWeightMap && !StiffnessWeftHasWeightMap && !StiffnessBiasHasWeightMap && !DampingHasWeightMap)
		{
			const FSolverVec3 ExpStiffnessValue((FSolverReal)StiffnessWeft, (FSolverReal)StiffnessWarp, (FSolverReal)StiffnessBias);
			const FSolverReal DampingRatioValue = (FSolverReal)DampingRatio;
			if (ExpStiffnessValue.Max() < MinStiffness)
			{
				return;
			}
			for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
			{
				const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
				const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
				PhysicsParallelFor(ColorSize, [this, &Particles, Dt, &LinearSystem, ColorStart, ExpStiffnessValue, DampingRatioValue](const int32 Index)
				{
					const int32 ConstraintIndex = ColorStart + Index;
					const FSolverVec3& WarpWeftBiasBaseMultiplier = WarpWeftBiasBaseMultipliers[ConstraintIndex];
					const FSolverReal FinalStiffnessValue = WarpWeftBiasBaseMultiplier.Dot(ExpStiffnessValue);
					Spring::UpdateSpringLinearSystem(Particles, Dt, Constraints[ConstraintIndex], Dists[ConstraintIndex], FinalStiffnessValue, MinStiffness, DampingRatioValue, LinearSystem);
				});
			}
		}
		else // Has weight maps
		{
			const FSolverReal StiffnessNoMap = (FSolverReal)StiffnessWarp;
			const FSolverReal StiffnessWeftNoMap = (FSolverReal)StiffnessWeft;
			const FSolverReal StiffnessBiasNoMap = (FSolverReal)StiffnessBias;
			const FSolverReal DampingNoMap = (FSolverReal)DampingRatio;
			for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
			{
				const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
				const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
				PhysicsParallelFor(ColorSize, [this, &Particles, Dt, &LinearSystem, ColorStart, StiffnessHasWeightMap, StiffnessNoMap, StiffnessWeftHasWeightMap, StiffnessWeftNoMap, StiffnessBiasHasWeightMap,
					StiffnessBiasNoMap, DampingHasWeightMap, DampingNoMap](const int32 Index)
				{
					const int32 ConstraintIndex = ColorStart + Index;
					const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? StiffnessWarp[ConstraintIndex] : StiffnessNoMap;
					const FSolverReal ExpStiffnessWeftValue = StiffnessWeftHasWeightMap ? StiffnessWeft[ConstraintIndex] : StiffnessWeftNoMap;
					const FSolverReal ExpStiffnessBiasValue = StiffnessBiasHasWeightMap ? StiffnessBias[ConstraintIndex] : StiffnessBiasNoMap;
					const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
					const FSolverVec3& WarpWeftBiasBaseMultiplier = WarpWeftBiasBaseMultipliers[ConstraintIndex];
					const FSolverReal FinalStiffnessValue = WarpWeftBiasBaseMultiplier.Dot(FSolverVec3(ExpStiffnessWeftValue, ExpStiffnessValue, ExpStiffnessBiasValue));
					Spring::UpdateSpringLinearSystem(Particles, Dt, Constraints[ConstraintIndex], Dists[ConstraintIndex], FinalStiffnessValue, MinStiffness, DampingRatioValue, LinearSystem);
				});
			}
		}
	}
	else
	{
		if (!StiffnessHasWeightMap && !StiffnessWeftHasWeightMap && !StiffnessBiasHasWeightMap && !DampingHasWeightMap)
		{
			const FSolverVec3 ExpStiffnessValue((FSolverReal)StiffnessWeft, (FSolverReal)StiffnessWarp, (FSolverReal)StiffnessBias);
			const FSolverReal DampingRatioValue = (FSolverReal)DampingRatio;
			if (ExpStiffnessValue.Max() < MinStiffness)
			{
				return;
			}
			for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
			{
				const FSolverVec3& WarpWeftBiasBaseMultiplier = WarpWeftBiasBaseMultipliers[ConstraintIndex];
				const FSolverReal FinalStiffnessValue = WarpWeftBiasBaseMultiplier.Dot(ExpStiffnessValue);
				Spring::UpdateSpringLinearSystem(Particles, Dt, Constraints[ConstraintIndex], Dists[ConstraintIndex], FinalStiffnessValue, MinStiffness, DampingRatioValue, LinearSystem);
			}
		}
		else
		{
			const FSolverReal StiffnessNoMap = (FSolverReal)StiffnessWarp;
			const FSolverReal StiffnessWeftNoMap = (FSolverReal)StiffnessWeft;
			const FSolverReal StiffnessBiasNoMap = (FSolverReal)StiffnessBias;
			const FSolverReal DampingNoMap = (FSolverReal)DampingRatio;
			for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
			{
				const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? StiffnessWarp[ConstraintIndex] : StiffnessNoMap;
				const FSolverReal ExpStiffnessWeftValue = StiffnessWeftHasWeightMap ? StiffnessWeft[ConstraintIndex] : StiffnessWeftNoMap;
				const FSolverReal ExpStiffnessBiasValue = StiffnessBiasHasWeightMap ? StiffnessBias[ConstraintIndex] : StiffnessBiasNoMap;
				const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
				const FSolverVec3& WarpWeftBiasBaseMultiplier = WarpWeftBiasBaseMultipliers[ConstraintIndex];
				const FSolverReal FinalStiffnessValue = WarpWeftBiasBaseMultiplier.Dot(FSolverVec3(ExpStiffnessWeftValue, ExpStiffnessValue, ExpStiffnessBiasValue));
				Spring::UpdateSpringLinearSystem(Particles, Dt, Constraints[ConstraintIndex], Dists[ConstraintIndex], FinalStiffnessValue, MinStiffness, DampingRatioValue, LinearSystem);
			}
		}
	}
}

FXPBDAnisotropicAxialSpringConstraints::FXPBDAnisotropicAxialSpringConstraints(
	const FSolverParticlesRange& Particles,
	const FTriangleMesh& TriangleMesh,
	const TArray<TVec3<FVec2f>>& FaceVertexPatternPositions,
	bool bUse3dRestLengths,
	bool SavePreResizedSimPosition3D,
	const TArray<FVector3f>& PreResizedSimPosition3D,
	FSolverReal MeshScale,
	const TConstArrayView<FRealSingle>& StiffnessWarpMultipliers,
	const TConstArrayView<FRealSingle>& StiffnessWeftMultipliers,
	const TConstArrayView<FRealSingle>& StiffnessBiasMultipliers,
	const TConstArrayView<FRealSingle>& DampingMultipliers,
	const TConstArrayView<FRealSingle>& WarpScaleMultipliers,
	const TConstArrayView<FRealSingle>& WeftScaleMultipliers,
	const FSolverVec2& InStiffnessWarp,
	const FSolverVec2& InStiffnessWeft,
	const FSolverVec2& InStiffnessBias,
	const FSolverVec2& InDampingRatio,
	const FSolverVec2& InWarpScale,
	const FSolverVec2& InWeftScale)
	: Base(Particles,
		TriangleMesh.GetElements(),
		TConstArrayView<FRealSingle>(), // We don't use base stiffness maps
		InStiffnessWarp,
		true /*bTrimKinematicConstraints*/,
		MaxStiffness)
	, StiffnessWarp(InStiffnessWarp.ClampAxes(0, MaxStiffness),
		StiffnessWarpMultipliers,
		TConstArrayView<TVec3<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, StiffnessWeft(InStiffnessWeft.ClampAxes(0, MaxStiffness),
		StiffnessWeftMultipliers,
		TConstArrayView<TVec3<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, StiffnessBias(InStiffnessBias.ClampAxes(0, MaxStiffness),
		StiffnessBiasMultipliers,
		TConstArrayView<TVec3<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, DampingRatio(
		InDampingRatio.ClampAxes(MinDampingRatio, MaxDampingRatio),
		DampingMultipliers,
		TConstArrayView<TVec3<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, WarpScale(
		InWarpScale.ClampAxes(MinWarpWeftScale, MaxWarpWeftScale),
		WarpScaleMultipliers,
		TConstArrayView<TVec3<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, WeftScale(
		InWeftScale.ClampAxes(MinWarpWeftScale, MaxWarpWeftScale),
		WeftScaleMultipliers,
		TConstArrayView<TVec3<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
{
	Init(); // Lambda initialization
	InitFromPatternData(Particles.Size(), bUse3dRestLengths, FaceVertexPatternPositions, TriangleMesh, SavePreResizedSimPosition3D, PreResizedSimPosition3D, MeshScale);
	InitColor(Particles);
}

FXPBDAnisotropicAxialSpringConstraints::FXPBDAnisotropicAxialSpringConstraints(
	const FSolverParticles& Particles,
	int32 InParticleOffset,
	int32 InParticleCount,
	const FTriangleMesh& TriangleMesh,
	const TArray<TVec3<FVec2f>>& FaceVertexPatternPositions,
	bool bUse3dRestLengths,
	bool SavePreResizedSimPosition3D,
	const TArray<FVector3f>& PreResizedSimPosition3D,
	FSolverReal MeshScale,
	const TConstArrayView<FRealSingle>& StiffnessWarpMultipliers,
	const TConstArrayView<FRealSingle>& StiffnessWeftMultipliers,
	const TConstArrayView<FRealSingle>& StiffnessBiasMultipliers,
	const TConstArrayView<FRealSingle>& DampingMultipliers,
	const TConstArrayView<FRealSingle>& WarpScaleMultipliers,
	const TConstArrayView<FRealSingle>& WeftScaleMultipliers,
	const FSolverVec2& InStiffnessWarp,
	const FSolverVec2& InStiffnessWeft,
	const FSolverVec2& InStiffnessBias,
	const FSolverVec2& InDampingRatio,
	const FSolverVec2& InWarpScale,
	const FSolverVec2& InWeftScale)
	: Base(Particles,
		InParticleOffset,
		InParticleCount,
		TriangleMesh.GetElements(),
		TConstArrayView<FRealSingle>(), // We don't use base stiffness maps
		InStiffnessWarp,
		true /*bTrimKinematicConstraints*/,
		MaxStiffness)
	, StiffnessWarp(InStiffnessWarp.ClampAxes(0, MaxStiffness),
		StiffnessWarpMultipliers,
		TConstArrayView<TVec3<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, StiffnessWeft(InStiffnessWeft.ClampAxes(0, MaxStiffness),
		StiffnessWeftMultipliers,
		TConstArrayView<TVec3<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, StiffnessBias(InStiffnessBias.ClampAxes(0, MaxStiffness),
		StiffnessBiasMultipliers,
		TConstArrayView<TVec3<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, DampingRatio(
		InDampingRatio.ClampAxes(MinDampingRatio, MaxDampingRatio),
		DampingMultipliers,
		TConstArrayView<TVec3<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, WarpScale(
		InWarpScale.ClampAxes(MinWarpWeftScale, MaxWarpWeftScale),
		WarpScaleMultipliers,
		TConstArrayView<TVec3<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, WeftScale(
		InWeftScale.ClampAxes(MinWarpWeftScale, MaxWarpWeftScale),
		WeftScaleMultipliers,
		TConstArrayView<TVec3<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
{
	Init(); // Lambda initialization
	InitFromPatternData(Particles.Size(), bUse3dRestLengths, FaceVertexPatternPositions, TriangleMesh, SavePreResizedSimPosition3D, PreResizedSimPosition3D, MeshScale);
	InitColor(Particles);
}

FXPBDAnisotropicAxialSpringConstraints::FXPBDAnisotropicAxialSpringConstraints(
	const FSolverParticlesRange& Particles,
	const FTriangleMesh& TriangleMesh,
	const TArray<TVec3<FVec2f>>& FaceVertexPatternPositions,
	bool bUse3dRestLengths,
	const TConstArrayView<FRealSingle>& StiffnessWarpMultipliers,
	const TConstArrayView<FRealSingle>& StiffnessWeftMultipliers,
	const TConstArrayView<FRealSingle>& StiffnessBiasMultipliers,
	const TConstArrayView<FRealSingle>& DampingMultipliers,
	const TConstArrayView<FRealSingle>& WarpScaleMultipliers,
	const TConstArrayView<FRealSingle>& WeftScaleMultipliers,
	const FSolverVec2& InStiffnessWarp,
	const FSolverVec2& InStiffnessWeft,
	const FSolverVec2& InStiffnessBias,
	const FSolverVec2& InDampingRatio,
	const FSolverVec2& InWarpScale,
	const FSolverVec2& InWeftScale)
	: Base(Particles,
		TriangleMesh.GetElements(),
		TConstArrayView<FRealSingle>(), // We don't use base stiffness maps
		InStiffnessWarp,
		true /*bTrimKinematicConstraints*/,
		MaxStiffness)
	, StiffnessWarp(InStiffnessWarp.ClampAxes(0, MaxStiffness),
		StiffnessWarpMultipliers,
		TConstArrayView<TVec3<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, StiffnessWeft(InStiffnessWeft.ClampAxes(0, MaxStiffness),
		StiffnessWeftMultipliers,
		TConstArrayView<TVec3<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, StiffnessBias(InStiffnessBias.ClampAxes(0, MaxStiffness),
		StiffnessBiasMultipliers,
		TConstArrayView<TVec3<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, DampingRatio(
		InDampingRatio.ClampAxes(MinDampingRatio, MaxDampingRatio),
		DampingMultipliers,
		TConstArrayView<TVec3<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, WarpScale(
		InWarpScale.ClampAxes(MinWarpWeftScale, MaxWarpWeftScale),
		WarpScaleMultipliers,
		TConstArrayView<TVec3<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, WeftScale(
		InWeftScale.ClampAxes(MinWarpWeftScale, MaxWarpWeftScale),
		WeftScaleMultipliers,
		TConstArrayView<TVec3<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
{
	Init(); // Lambda initialization
	InitFromPatternData(Particles.Size(), bUse3dRestLengths, FaceVertexPatternPositions, TriangleMesh, /*SavePreResizedSimPosition3D = */false, /*PreResizedSimPosition3D = */TArray<FVector3f>(), /*MeshScale = */ (FSolverReal)1.);
	InitColor(Particles);
}

FXPBDAnisotropicAxialSpringConstraints::FXPBDAnisotropicAxialSpringConstraints(
	const FSolverParticles& Particles,
	int32 InParticleOffset,
	int32 InParticleCount,
	const FTriangleMesh& TriangleMesh,
	const TArray<TVec3<FVec2f>>& FaceVertexPatternPositions,
	bool bUse3dRestLengths,
	const TConstArrayView<FRealSingle>& StiffnessWarpMultipliers,
	const TConstArrayView<FRealSingle>& StiffnessWeftMultipliers,
	const TConstArrayView<FRealSingle>& StiffnessBiasMultipliers,
	const TConstArrayView<FRealSingle>& DampingMultipliers,
	const TConstArrayView<FRealSingle>& WarpScaleMultipliers,
	const TConstArrayView<FRealSingle>& WeftScaleMultipliers,
	const FSolverVec2& InStiffnessWarp,
	const FSolverVec2& InStiffnessWeft,
	const FSolverVec2& InStiffnessBias,
	const FSolverVec2& InDampingRatio,
	const FSolverVec2& InWarpScale,
	const FSolverVec2& InWeftScale)
	: Base(Particles,
		InParticleOffset,
		InParticleCount,
		TriangleMesh.GetElements(),
		TConstArrayView<FRealSingle>(), // We don't use base stiffness maps
		InStiffnessWarp,
		true /*bTrimKinematicConstraints*/,
		MaxStiffness)
	, StiffnessWarp(InStiffnessWarp.ClampAxes(0, MaxStiffness),
		StiffnessWarpMultipliers,
		TConstArrayView<TVec3<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, StiffnessWeft(InStiffnessWeft.ClampAxes(0, MaxStiffness),
		StiffnessWeftMultipliers,
		TConstArrayView<TVec3<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, StiffnessBias(InStiffnessBias.ClampAxes(0, MaxStiffness),
		StiffnessBiasMultipliers,
		TConstArrayView<TVec3<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, DampingRatio(
		InDampingRatio.ClampAxes(MinDampingRatio, MaxDampingRatio),
		DampingMultipliers,
		TConstArrayView<TVec3<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, WarpScale(
		InWarpScale.ClampAxes(MinWarpWeftScale, MaxWarpWeftScale),
		WarpScaleMultipliers,
		TConstArrayView<TVec3<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, WeftScale(
		InWeftScale.ClampAxes(MinWarpWeftScale, MaxWarpWeftScale),
		WeftScaleMultipliers,
		TConstArrayView<TVec3<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
{
	Init(); // Lambda initialization
	InitFromPatternData(Particles.Size(), bUse3dRestLengths, FaceVertexPatternPositions, TriangleMesh, /*SavePreResizedSimPosition3D = */false, /*PreResizedSimPosition3D = */TArray<FVector3f>(), /*MeshScale = */ (FSolverReal)1.);
	InitColor(Particles);
}

void FXPBDAnisotropicAxialSpringConstraints::InitFromPatternData(int32 ParticlesSize, bool bUse3dRestLengths, const TArray<TVec3<FVec2f>>& FaceVertexPatternPositions, const FTriangleMesh& TriangleMesh, bool SavePreResizedSimPosition3D, const TArray<FVector3f>& PreResizedSimPosition3D, FSolverReal MeshScale)
{
	if (bUse3dRestLengths)
	{
		BaseDists = Dists; // Use Dists calculated by base class using 3D positions
	}
	else
	{
		// We'll set them from computed 2D lengths
		BaseDists.SetNumUninitialized(Constraints.Num());
	}
	WarpWeftBiasBaseMultipliers.SetNumUninitialized(Constraints.Num());
	WarpWeftScaleBaseMultipliers.SetNumUninitialized(Constraints.Num());

	TConstArrayView<TArray<int32>> PointToTriangleMap = TriangleMesh.GetPointToTriangleMap();
	auto SortedElement = [](const TVec3<int32>& Element)
	{
		return TVec3<int32>(Element.Min(), Element.Mid(), Element.Max());
	};
	const TArray<TVec3<int32>>& Elements = TriangleMesh.GetElements();

	for (int32 ConstraintIdx = 0; ConstraintIdx < Constraints.Num(); ++ConstraintIdx)
	{
		const TVec3<int32>& Constraint = Constraints[ConstraintIdx];
		// Find corresponding triangle in original mesh (constraints get reordered by Base class init)
		const TArray<int32>& PossibleTriangles = PointToTriangleMap[Constraint[0]];
		const TVec3<int32> SortedConstraint = SortedElement(Constraint);
		int32 TriangleIndex = INDEX_NONE;
		for (const int32 Triangle : PossibleTriangles)
		{
			if (SortedConstraint == SortedElement(Elements[Triangle]))
			{
				TriangleIndex = Triangle;
				break;
			}
		}
		check(TriangleIndex != INDEX_NONE);

		const TVec3<int32>& Element = Elements[TriangleIndex];
		auto MatchIndex = [&Constraint, &Element](const int32 ConstraintAxis)
		{
			check(Element[0] == Constraint[ConstraintAxis] || Element[1] == Constraint[ConstraintAxis] || Element[2] == Constraint[ConstraintAxis]);
			return Element[0] == Constraint[ConstraintAxis] ? 0 : Element[1] == Constraint[ConstraintAxis] ? 1 : 2;
		};
		const int32 FaceIndex1 = MatchIndex(0);
		const int32 FaceIndex2 = MatchIndex(1);
		const int32 FaceIndex3 = MatchIndex(2);

		const FVec2f& UV1 = FaceVertexPatternPositions[TriangleIndex][FaceIndex1];
		const FVec2f& UV2 = FaceVertexPatternPositions[TriangleIndex][FaceIndex2];
		const FVec2f& UV3 = FaceVertexPatternPositions[TriangleIndex][FaceIndex3];

		const FSolverVec2 UVDiff = GetSpringDiff(UV1, UV2, UV3, Barys[ConstraintIdx]);
		const FSolverVec2 UVDiffAbs = UVDiff.GetAbs();
		FSolverVec3 UVDiffTransformed;
		if (UVDiffAbs.X > UVDiffAbs.Y)
		{
			// Stiffness is blend between weft and bias
			UVDiffTransformed = FSolverVec3(UVDiffAbs.X - UVDiffAbs.Y, (FSolverReal)0.f, UVDiffAbs.Y);
		}
		else
		{
			// Stiffness is blend between warp and bias
			UVDiffTransformed = FSolverVec3((FSolverReal)0.f, UVDiffAbs.Y - UVDiffAbs.X, UVDiffAbs.X);
		}
		const FSolverReal Denom3 = UVDiffTransformed.X + UVDiffTransformed.Y + UVDiffTransformed.Z; // These are used as weights
		const FSolverReal UVLength = UVDiffAbs.Length();
		WarpWeftBiasBaseMultipliers[ConstraintIdx] = Denom3 > UE_SMALL_NUMBER ? UVDiffTransformed / Denom3 : FSolverVec3(0.f, 1.f, 0.f); // Default to Warp if zero length
		WarpWeftScaleBaseMultipliers[ConstraintIdx] = UVLength > UE_SMALL_NUMBER ? UVDiffAbs / UVLength : FSolverVec2(UE_INV_SQRT_2, UE_INV_SQRT_2); // Default to equally scaling warp and weft directions if zero length

		if (!bUse3dRestLengths)
		{
			BaseDists[ConstraintIdx] = UVLength;
			if (SavePreResizedSimPosition3D && (ParticlesSize == PreResizedSimPosition3D.Num()))
			{
				const FSolverReal Denom = MeshScale * GetSpringDiff(
					PreResizedSimPosition3D[Constraint[0]], 
					PreResizedSimPosition3D[Constraint[1]], 
					PreResizedSimPosition3D[Constraint[2]], 
					Barys[ConstraintIdx]).Size();
				const FSolverReal Scale = Denom > UE_SMALL_NUMBER ? Dists[ConstraintIdx] / Denom : 1;
				BaseDists[ConstraintIdx] *= Scale;
			}
			Dists[ConstraintIdx] = BaseDists[ConstraintIdx];
		}
	}
}

template<typename SolverParticlesOrRange>
void FXPBDAnisotropicAxialSpringConstraints::InitColor(const SolverParticlesOrRange& InParticles)
{
	// In dev builds we always color so we can tune the system without restarting. See Apply()
#if UE_BUILD_SHIPPING || UE_BUILD_TEST
	if (Constraints.Num() > Chaos_XPBDSpring_ParallelConstraintCount)
#endif
	{
		const TArray<TArray<int32>> ConstraintsPerColor = FGraphColoring::ComputeGraphColoringParticlesOrRange(Constraints, InParticles, ParticleOffset, ParticleOffset + ParticleCount);

		// Reorder constraints based on color so each array in ConstraintsPerColor contains contiguous elements.
		TArray<TVec3<int32>> ReorderedConstraints;
		TArray<FSolverReal> ReorderedBarys;
		TArray<FSolverReal> ReorderedDists;
		TArray<FSolverReal> ReorderedBaseDists;
		TArray<FSolverVec3> ReorderedWarpWeftBiasBaseMultipliers;
		TArray<FSolverVec2> ReorderedWarpWeftScaleBaseMultipliers;
		TArray<int32> OrigToReorderedIndices; // used to reorder stiffness indices
		ReorderedConstraints.SetNumUninitialized(Constraints.Num());
		ReorderedBarys.SetNumUninitialized(Constraints.Num());
		ReorderedDists.SetNumUninitialized(Constraints.Num());
		ReorderedBaseDists.SetNumUninitialized(Constraints.Num());
		ReorderedWarpWeftBiasBaseMultipliers.SetNumUninitialized(Constraints.Num());
		ReorderedWarpWeftScaleBaseMultipliers.SetNumUninitialized(Constraints.Num());
		OrigToReorderedIndices.SetNumUninitialized(Constraints.Num());

		ConstraintsPerColorStartIndex.Reset(ConstraintsPerColor.Num() + 1);

		int32 ReorderedIndex = 0;
		for (const TArray<int32>& ConstraintsBatch : ConstraintsPerColor)
		{
			ConstraintsPerColorStartIndex.Add(ReorderedIndex);
			for (const int32& BatchConstraint : ConstraintsBatch)
			{
				const int32 OrigIndex = BatchConstraint;
				ReorderedConstraints[ReorderedIndex] = Constraints[OrigIndex];
				ReorderedBarys[ReorderedIndex] = Barys[OrigIndex];
				ReorderedDists[ReorderedIndex] = Dists[OrigIndex];
				ReorderedBaseDists[ReorderedIndex] = BaseDists[OrigIndex];
				ReorderedWarpWeftBiasBaseMultipliers[ReorderedIndex] = WarpWeftBiasBaseMultipliers[OrigIndex];
				ReorderedWarpWeftScaleBaseMultipliers[ReorderedIndex] = WarpWeftScaleBaseMultipliers[OrigIndex];
				OrigToReorderedIndices[OrigIndex] = ReorderedIndex;
				++ReorderedIndex;
			}
		}
		ConstraintsPerColorStartIndex.Add(ReorderedIndex);

		Constraints = MoveTemp(ReorderedConstraints);
		Barys = MoveTemp(ReorderedBarys);
		Dists = MoveTemp(ReorderedDists);
		BaseDists = MoveTemp(ReorderedBaseDists);
		WarpWeftBiasBaseMultipliers = MoveTemp(ReorderedWarpWeftBiasBaseMultipliers);
		WarpWeftScaleBaseMultipliers = MoveTemp(ReorderedWarpWeftScaleBaseMultipliers);
		Stiffness.ReorderIndices(OrigToReorderedIndices);
		StiffnessWeft.ReorderIndices(OrigToReorderedIndices);
		StiffnessBias.ReorderIndices(OrigToReorderedIndices);
		DampingRatio.ReorderIndices(OrigToReorderedIndices);
		WarpScale.ReorderIndices(OrigToReorderedIndices);
		WeftScale.ReorderIndices(OrigToReorderedIndices);
	}
}

void FXPBDAnisotropicAxialSpringConstraints::UpdateDists()
{
	Private::UpdateDists(BaseDists, WarpWeftScaleBaseMultipliers, WarpScale, WeftScale, Dists);
}

template<bool bDampingBefore, bool bSingleLambda, bool bSeparateStretch, bool bDampingAfter, typename SolverParticlesOrRange>
void FXPBDAnisotropicAxialSpringConstraints::ApplyHelper(SolverParticlesOrRange& Particles, const FSolverReal Dt, const int32 ConstraintIndex, const FSolverVec3& ExpStiffnessValues, const FSolverReal DampingRatioValue) const
{
	const TVec3<int32>& Constraint = Constraints[ConstraintIndex];
	const int32 Index1 = Constraint[0];
	const int32 Index2 = Constraint[1];
	const int32 Index3 = Constraint[2];

	const FSolverVec3& WarpWeftBiasBaseMultiplier = WarpWeftBiasBaseMultipliers[ConstraintIndex];
	const FSolverReal ExpStiffnessValue = WarpWeftBiasBaseMultiplier.Dot(ExpStiffnessValues);

	FSolverVec3 Delta(0.f);
	if constexpr (bDampingBefore)
	{
		Delta += Spring::GetXPBDAxialSpringDampingDelta(Particles, Dt, Constraint, Barys[ConstraintIndex], Dists[ConstraintIndex], LambdasDamping[ConstraintIndex],
			ExpStiffnessValue, DampingRatioValue);
	}

	if constexpr (bSingleLambda)
	{
		Delta += Spring::GetXPBDAxialSpringDeltaWithDamping(Particles, Dt, Constraint, Barys[ConstraintIndex], Dists[ConstraintIndex], Lambdas[ConstraintIndex],
			ExpStiffnessValue, DampingRatioValue);
	}

	if constexpr (bSeparateStretch)
	{
		Delta += Spring::GetXPBDAxialSpringDelta(Particles, Dt, Constraint, Barys[ConstraintIndex], Dists[ConstraintIndex], Lambdas[ConstraintIndex],
			ExpStiffnessValue);
	}

	if constexpr (bDampingAfter)
	{
		Delta += Spring::GetXPBDAxialSpringDampingDelta(Particles, Dt, Constraint, Barys[ConstraintIndex], Dists[ConstraintIndex], LambdasDamping[ConstraintIndex],
			ExpStiffnessValue, DampingRatioValue);
	}
	Particles.P(Index1) += Particles.InvM(Index1) * Delta;
	Particles.P(Index2) -= Particles.InvM(Index2) * Barys[ConstraintIndex] * Delta;
	Particles.P(Index3) -= Particles.InvM(Index3) * ((FSolverReal)1. - Barys[ConstraintIndex]) * Delta;
}

template<typename SolverParticlesOrRange>
void FXPBDAnisotropicAxialSpringConstraints::Apply(SolverParticlesOrRange& Particles, const FSolverReal Dt) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FXPBDAnisotropicAxialSpringConstraints_Apply);
	SCOPE_CYCLE_COUNTER(STAT_XPBD_AnisoSpring);

	const bool StiffnessHasWeightMap = StiffnessWarp.HasWeightMap();
	const bool StiffnessWeftHasWeightMap = StiffnessWeft.HasWeightMap();
	const bool StiffnessBiasHasWeightMap = StiffnessBias.HasWeightMap();
	const bool DampingHasWeightMap = DampingRatio.HasWeightMap();

	if ((ConstraintsPerColorStartIndex.Num() > 1) && (Constraints.Num() > Chaos_XPBDSpring_ParallelConstraintCount))
	{
		const int32 ConstraintColorNum = ConstraintsPerColorStartIndex.Num() - 1;
#if INTEL_ISPC
		if (bRealTypeCompatibleWithISPC && bChaos_XPBDSpring_ISPC_Enabled)
		{
			const bool bSingleLambda = Chaos_XPBDSpring_SplitDampingMode == (int32)EXPBDSplitDampingMode::SingleLambda;

			if (!StiffnessHasWeightMap && !StiffnessWeftHasWeightMap && !StiffnessBiasHasWeightMap && !DampingHasWeightMap)
			{
				const FSolverVec3 ExpStiffnessValue((FSolverReal)StiffnessWeft, (FSolverReal)StiffnessWarp, (FSolverReal)StiffnessBias);
				const FSolverReal DampingRatioValue = (FSolverReal)DampingRatio;
				if (ExpStiffnessValue.Max() <= MinStiffness)
				{
					return;
				}

				if (DampingRatioValue > 0)
				{
					if (bSingleLambda)
					{
						for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
						{
							const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
							const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
							ispc::ApplyXPBDAnisoAxialSpringConstraintsWithDamping(
								(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
								(const ispc::FVector3f*)Particles.XArray().GetData(),
								(ispc::FIntVector*)&Constraints.GetData()[ColorStart],
								&Barys.GetData()[ColorStart],
								&Dists.GetData()[ColorStart],
								(const ispc::FVector3f*)&WarpWeftBiasBaseMultipliers.GetData()[ColorStart],
								&LambdasDamping.GetData()[ColorStart],
								Dt,
								reinterpret_cast<const ispc::FVector3f&>(ExpStiffnessValue),
								DampingRatioValue,
								ColorSize);
						}
						return;
					}
					else
					{
						for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
						{
							const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
							const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
							ispc::ApplyXPBDAnisoAxialSpringDampingConstraints(
								(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
								(const ispc::FVector3f*)Particles.XArray().GetData(),
								(ispc::FIntVector*)&Constraints.GetData()[ColorStart],
								&Barys.GetData()[ColorStart],
								&Dists.GetData()[ColorStart],
								(const ispc::FVector3f*)&WarpWeftBiasBaseMultipliers.GetData()[ColorStart],
								&LambdasDamping.GetData()[ColorStart],
								Dt,
								reinterpret_cast<const ispc::FVector3f&>(ExpStiffnessValue),
								DampingRatioValue,
								ColorSize);
						}
					}
				}
				for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
				{
					const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
					const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
					ispc::ApplyXPBDAnisoAxialSpringConstraints(
						(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
						(ispc::FIntVector*)&Constraints.GetData()[ColorStart],
						&Barys.GetData()[ColorStart],
						&Dists.GetData()[ColorStart],
						(const ispc::FVector3f*)&WarpWeftBiasBaseMultipliers.GetData()[ColorStart],
						&Lambdas.GetData()[ColorStart],
						Dt,
						reinterpret_cast<const ispc::FVector3f&>(ExpStiffnessValue),
						ColorSize);
				}
			}
			else // ISPC with maps
			{
				if (DampingHasWeightMap || (FSolverReal)DampingRatio > 0)
				{
					if (bSingleLambda)
					{
						for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
						{
							const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
							const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
							ispc::ApplyXPBDAnisoAxialSpringConstraintsWithDampingAndWeightMaps(
								(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
								(const ispc::FVector3f*)Particles.XArray().GetData(),
								(ispc::FIntVector*)&Constraints.GetData()[ColorStart],
								&Barys.GetData()[ColorStart],
								&Dists.GetData()[ColorStart],
								(const ispc::FVector3f*)&WarpWeftBiasBaseMultipliers.GetData()[ColorStart],
								&LambdasDamping.GetData()[ColorStart],
								Dt,
								StiffnessHasWeightMap,
								reinterpret_cast<const ispc::FVector2f&>(StiffnessWarp.GetOffsetRange()),
								StiffnessHasWeightMap ? &StiffnessWarp.GetMapValues().GetData()[ColorStart] : nullptr,
								StiffnessWeftHasWeightMap,
								reinterpret_cast<const ispc::FVector2f&>(StiffnessWeft.GetOffsetRange()),
								StiffnessWeftHasWeightMap ? &StiffnessWeft.GetMapValues().GetData()[ColorStart] : nullptr,
								StiffnessBiasHasWeightMap,
								reinterpret_cast<const ispc::FVector2f&>(StiffnessBias.GetOffsetRange()),
								StiffnessBiasHasWeightMap ? &StiffnessBias.GetMapValues().GetData()[ColorStart] : nullptr,
								DampingHasWeightMap,
								reinterpret_cast<const ispc::FVector2f&>(DampingRatio.GetOffsetRange()),
								DampingHasWeightMap ? &DampingRatio.GetMapValues()[ColorStart] : nullptr,
								ColorSize);
						}
						return;
					}
					else
					{
						for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
						{
							const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
							const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
							ispc::ApplyXPBDAnisoAxialSpringDampingConstraintsWithWeightMaps(
								(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
								(const ispc::FVector3f*)Particles.XArray().GetData(),
								(ispc::FIntVector*)&Constraints.GetData()[ColorStart],
								&Barys.GetData()[ColorStart],
								&Dists.GetData()[ColorStart],
								(const ispc::FVector3f*)&WarpWeftBiasBaseMultipliers.GetData()[ColorStart],
								&LambdasDamping.GetData()[ColorStart],
								Dt,
								StiffnessHasWeightMap,
								reinterpret_cast<const ispc::FVector2f&>(StiffnessWarp.GetOffsetRange()),
								StiffnessHasWeightMap ? &StiffnessWarp.GetMapValues().GetData()[ColorStart] : nullptr,
								StiffnessWeftHasWeightMap,
								reinterpret_cast<const ispc::FVector2f&>(StiffnessWeft.GetOffsetRange()),
								StiffnessWeftHasWeightMap ? &StiffnessWeft.GetMapValues().GetData()[ColorStart] : nullptr,
								StiffnessBiasHasWeightMap,
								reinterpret_cast<const ispc::FVector2f&>(StiffnessBias.GetOffsetRange()),
								StiffnessBiasHasWeightMap ? &StiffnessBias.GetMapValues().GetData()[ColorStart] : nullptr,
								DampingHasWeightMap,
								reinterpret_cast<const ispc::FVector2f&>(DampingRatio.GetOffsetRange()),
								DampingHasWeightMap ? &DampingRatio.GetMapValues()[ColorStart] : nullptr,
								ColorSize);
						}
					}
				}
				for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
				{
					const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
					const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
					ispc::ApplyXPBDAnisoAxialSpringConstraintsWithWeightMaps(
						(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
						(ispc::FIntVector*)&Constraints.GetData()[ColorStart],
						&Barys.GetData()[ColorStart],
						&Dists.GetData()[ColorStart],
						(const ispc::FVector3f*)&WarpWeftBiasBaseMultipliers.GetData()[ColorStart],
						&Lambdas.GetData()[ColorStart],
						Dt,
						StiffnessHasWeightMap,
						reinterpret_cast<const ispc::FVector2f&>(StiffnessWarp.GetOffsetRange()),
						StiffnessHasWeightMap ? &StiffnessWarp.GetMapValues().GetData()[ColorStart] : nullptr,
						StiffnessWeftHasWeightMap,
						reinterpret_cast<const ispc::FVector2f&>(StiffnessWeft.GetOffsetRange()),
						StiffnessWeftHasWeightMap ? &StiffnessWeft.GetMapValues().GetData()[ColorStart] : nullptr,
						StiffnessBiasHasWeightMap,
						reinterpret_cast<const ispc::FVector2f&>(StiffnessBias.GetOffsetRange()),
						StiffnessBiasHasWeightMap ? &StiffnessBias.GetMapValues().GetData()[ColorStart] : nullptr,
						ColorSize);
				}
			}
		}
		else
#endif
		{
			// Parallel non-ispc
			const FSolverReal StiffnessNoMap = (FSolverReal)StiffnessWarp;
			const FSolverReal StiffnessWeftNoMap = (FSolverReal)StiffnessWeft;
			const FSolverReal StiffnessBiasNoMap = (FSolverReal)StiffnessBias;
			const FSolverReal DampingNoMap = (FSolverReal)DampingRatio;
			if (DampingNoMap > 0 || DampingHasWeightMap)
			{
				if (Chaos_XPBDSpring_SplitDampingMode == (int32)EXPBDSplitDampingMode::TwoPassBefore)
				{
					for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
					{
						const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
						const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
						PhysicsParallelFor(ColorSize, [this, &Particles, Dt, ColorStart, StiffnessHasWeightMap, StiffnessNoMap, StiffnessWeftHasWeightMap, StiffnessWeftNoMap, StiffnessBiasHasWeightMap, StiffnessBiasNoMap, DampingHasWeightMap, DampingNoMap](const int32 Index)
						{
							const int32 ConstraintIndex = ColorStart + Index;
							constexpr bool bDampingBefore = true;
							constexpr bool bSingleLambda = false;
							constexpr bool bSeparateStretch = false;
							constexpr bool bDampingAfter = false;
							const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? StiffnessWarp[ConstraintIndex] : StiffnessNoMap;
							const FSolverReal ExpStiffnessWeftValue = StiffnessWeftHasWeightMap ? StiffnessWeft[ConstraintIndex] : StiffnessWeftNoMap;
							const FSolverReal ExpStiffnessBiasValue = StiffnessBiasHasWeightMap ? StiffnessBias[ConstraintIndex] : StiffnessBiasNoMap;
							const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
							ApplyHelper<bDampingBefore, bSingleLambda, bSeparateStretch, bDampingAfter>(Particles, Dt, ConstraintIndex, FSolverVec3(ExpStiffnessWeftValue, ExpStiffnessValue, ExpStiffnessBiasValue), DampingRatioValue);
						});
					}
				}
				
				// Stretch part (possibly with damping depending on split mode)
				switch (Chaos_XPBDSpring_SplitDampingMode)
				{
				case (int32)EXPBDSplitDampingMode::TwoPassBefore: // fallthrough
				case (int32)EXPBDSplitDampingMode::TwoPassAfter: // fallthrough
				default:
				{
					// Do a pass with stretch only
					for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
					{
						const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
						const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
						PhysicsParallelFor(ColorSize, [this, &Particles, Dt, ColorStart, StiffnessHasWeightMap, StiffnessNoMap, StiffnessWeftHasWeightMap, StiffnessWeftNoMap, StiffnessBiasHasWeightMap, StiffnessBiasNoMap, DampingHasWeightMap, DampingNoMap](const int32 Index)
						{
							const int32 ConstraintIndex = ColorStart + Index;
							constexpr bool bDampingBefore = false;
							constexpr bool bSingleLambda = false;
							constexpr bool bSeparateStretch = true;
							constexpr bool bDampingAfter = false;
							const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? StiffnessWarp[ConstraintIndex] : StiffnessNoMap;
							const FSolverReal ExpStiffnessWeftValue = StiffnessWeftHasWeightMap ? StiffnessWeft[ConstraintIndex] : StiffnessWeftNoMap;
							const FSolverReal ExpStiffnessBiasValue = StiffnessBiasHasWeightMap ? StiffnessBias[ConstraintIndex] : StiffnessBiasNoMap;
							const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
							ApplyHelper<bDampingBefore, bSingleLambda, bSeparateStretch, bDampingAfter>(Particles, Dt, ConstraintIndex, FSolverVec3(ExpStiffnessWeftValue, ExpStiffnessValue, ExpStiffnessBiasValue), DampingRatioValue);
						});
					}
				}
				break;
				case (int32)EXPBDSplitDampingMode::SingleLambda:
				{
					// Do a pass with combined stretch and damping with a single lambda
					for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
					{
						const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
						const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
						PhysicsParallelFor(ColorSize, [this, &Particles, Dt, ColorStart, StiffnessHasWeightMap, StiffnessNoMap, StiffnessWeftHasWeightMap, StiffnessWeftNoMap, StiffnessBiasHasWeightMap, StiffnessBiasNoMap, DampingHasWeightMap, DampingNoMap](const int32 Index)
						{
							const int32 ConstraintIndex = ColorStart + Index;
							constexpr bool bDampingBefore = false;
							constexpr bool bSingleLambda = true;
							constexpr bool bSeparateStretch = false;
							constexpr bool bDampingAfter = false;
							const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? StiffnessWarp[ConstraintIndex] : StiffnessNoMap;
							const FSolverReal ExpStiffnessWeftValue = StiffnessWeftHasWeightMap ? StiffnessWeft[ConstraintIndex] : StiffnessWeftNoMap;
							const FSolverReal ExpStiffnessBiasValue = StiffnessBiasHasWeightMap ? StiffnessBias[ConstraintIndex] : StiffnessBiasNoMap;
							const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
							ApplyHelper<bDampingBefore, bSingleLambda, bSeparateStretch, bDampingAfter>(Particles, Dt, ConstraintIndex, FSolverVec3(ExpStiffnessWeftValue, ExpStiffnessValue, ExpStiffnessBiasValue), DampingRatioValue);
						});
					}
				}
				break;
				case (int32)EXPBDSplitDampingMode::InterleavedBefore:
				{
					// Do a pass with damping before stretch
					for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
					{
						const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
						const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
						PhysicsParallelFor(ColorSize, [this, &Particles, Dt, ColorStart, StiffnessHasWeightMap, StiffnessNoMap, StiffnessWeftHasWeightMap, StiffnessWeftNoMap, StiffnessBiasHasWeightMap, StiffnessBiasNoMap, DampingHasWeightMap, DampingNoMap](const int32 Index)
						{
							const int32 ConstraintIndex = ColorStart + Index;
							constexpr bool bDampingBefore = true;
							constexpr bool bSingleLambda = false;
							constexpr bool bSeparateStretch = true;
							constexpr bool bDampingAfter = false;
							const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? StiffnessWarp[ConstraintIndex] : StiffnessNoMap;
							const FSolverReal ExpStiffnessWeftValue = StiffnessWeftHasWeightMap ? StiffnessWeft[ConstraintIndex] : StiffnessWeftNoMap;
							const FSolverReal ExpStiffnessBiasValue = StiffnessBiasHasWeightMap ? StiffnessBias[ConstraintIndex] : StiffnessBiasNoMap;
							const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
							ApplyHelper<bDampingBefore, bSingleLambda, bSeparateStretch, bDampingAfter>(Particles, Dt, ConstraintIndex, FSolverVec3(ExpStiffnessWeftValue, ExpStiffnessValue, ExpStiffnessBiasValue), DampingRatioValue);
						});
					}
				}
				break;
				case (int32)EXPBDSplitDampingMode::InterleavedAfter:
				{
					// Do a pass with damping after stretch
					for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
					{
						const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
						const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
						PhysicsParallelFor(ColorSize, [this, &Particles, Dt, ColorStart, StiffnessHasWeightMap, StiffnessNoMap, StiffnessWeftHasWeightMap, StiffnessWeftNoMap, StiffnessBiasHasWeightMap, StiffnessBiasNoMap, DampingHasWeightMap, DampingNoMap](const int32 Index)
						{
							const int32 ConstraintIndex = ColorStart + Index;
							constexpr bool bDampingBefore = false;
							constexpr bool bSingleLambda = false;
							constexpr bool bSeparateStretch = true;
							constexpr bool bDampingAfter = true;
							const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? StiffnessWarp[ConstraintIndex] : StiffnessNoMap;
							const FSolverReal ExpStiffnessWeftValue = StiffnessWeftHasWeightMap ? StiffnessWeft[ConstraintIndex] : StiffnessWeftNoMap;
							const FSolverReal ExpStiffnessBiasValue = StiffnessBiasHasWeightMap ? StiffnessBias[ConstraintIndex] : StiffnessBiasNoMap;
							const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
							ApplyHelper<bDampingBefore, bSingleLambda, bSeparateStretch, bDampingAfter>(Particles, Dt, ConstraintIndex, FSolverVec3(ExpStiffnessWeftValue, ExpStiffnessValue, ExpStiffnessBiasValue), DampingRatioValue);
						});
					}
				}
				break;
				}
				if (Chaos_XPBDSpring_SplitDampingMode == (int32)EXPBDSplitDampingMode::TwoPassAfter)
				{
					for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
					{
						const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
						const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
						PhysicsParallelFor(ColorSize, [this, &Particles, Dt, ColorStart, StiffnessHasWeightMap, StiffnessNoMap, StiffnessWeftHasWeightMap, StiffnessWeftNoMap, StiffnessBiasHasWeightMap, StiffnessBiasNoMap, DampingHasWeightMap, DampingNoMap](const int32 Index)
						{
							const int32 ConstraintIndex = ColorStart + Index;
							constexpr bool bDampingBefore = false;
							constexpr bool bSingleLambda = false;
							constexpr bool bSeparateStretch = false;
							constexpr bool bDampingAfter = true;
							const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? StiffnessWarp[ConstraintIndex] : StiffnessNoMap;
							const FSolverReal ExpStiffnessWeftValue = StiffnessWeftHasWeightMap ? StiffnessWeft[ConstraintIndex] : StiffnessWeftNoMap;
							const FSolverReal ExpStiffnessBiasValue = StiffnessBiasHasWeightMap ? StiffnessBias[ConstraintIndex] : StiffnessBiasNoMap;
							const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
							ApplyHelper<bDampingBefore, bSingleLambda, bSeparateStretch, bDampingAfter>(Particles, Dt, ConstraintIndex, FSolverVec3(ExpStiffnessWeftValue, ExpStiffnessValue, ExpStiffnessBiasValue), DampingRatioValue);
						});
					}
				}
			}
			else
			{
				// No damping. 
				for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
				{
					const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
					const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
					PhysicsParallelFor(ColorSize, [this, &Particles, Dt, ColorStart, StiffnessHasWeightMap, StiffnessNoMap, StiffnessWeftHasWeightMap, StiffnessWeftNoMap, StiffnessBiasHasWeightMap, StiffnessBiasNoMap](const int32 Index)
					{
						const int32 ConstraintIndex = ColorStart + Index;
						constexpr bool bDampingBefore = false;
						constexpr bool bSingleLambda = false;
						constexpr bool bSeparateStretch = true;
						constexpr bool bDampingAfter = false;
						const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? StiffnessWarp[ConstraintIndex] : StiffnessNoMap;
						const FSolverReal ExpStiffnessWeftValue = StiffnessWeftHasWeightMap ? StiffnessWeft[ConstraintIndex] : StiffnessWeftNoMap;
						const FSolverReal ExpStiffnessBiasValue = StiffnessBiasHasWeightMap ? StiffnessBias[ConstraintIndex] : StiffnessBiasNoMap;
						ApplyHelper<bDampingBefore, bSingleLambda, bSeparateStretch, bDampingAfter>(Particles, Dt, ConstraintIndex, FSolverVec3(ExpStiffnessWeftValue, ExpStiffnessValue, ExpStiffnessBiasValue), (FSolverReal)0.);
					});
				}
			}
		}
	}
	else
	{
		// Single threaded
		const FSolverReal StiffnessNoMap = (FSolverReal)StiffnessWarp;
		const FSolverReal StiffnessWeftNoMap = (FSolverReal)StiffnessWeft;
		const FSolverReal StiffnessBiasNoMap = (FSolverReal)StiffnessBias;
		const FSolverReal DampingNoMap = (FSolverReal)DampingRatio;
		if (DampingNoMap > 0 || DampingHasWeightMap)
		{
			if (Chaos_XPBDSpring_SplitDampingMode == (int32)EXPBDSplitDampingMode::TwoPassBefore)
			{
				for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
				{
					constexpr bool bDampingBefore = true;
					constexpr bool bSingleLambda = false;
					constexpr bool bSeparateStretch = false;
					constexpr bool bDampingAfter = false;
					const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? StiffnessWarp[ConstraintIndex] : StiffnessNoMap;
					const FSolverReal ExpStiffnessWeftValue = StiffnessWeftHasWeightMap ? StiffnessWeft[ConstraintIndex] : StiffnessWeftNoMap;
					const FSolverReal ExpStiffnessBiasValue = StiffnessBiasHasWeightMap ? StiffnessBias[ConstraintIndex] : StiffnessBiasNoMap;
					const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
					ApplyHelper<bDampingBefore, bSingleLambda, bSeparateStretch, bDampingAfter>(Particles, Dt, ConstraintIndex, FSolverVec3(ExpStiffnessWeftValue, ExpStiffnessValue, ExpStiffnessBiasValue), DampingRatioValue);
				}
			}

			// Stretch part (possibly with damping depending on split mode)
			switch (Chaos_XPBDSpring_SplitDampingMode)
			{
			case (int32)EXPBDSplitDampingMode::TwoPassBefore: // fallthrough
			case (int32)EXPBDSplitDampingMode::TwoPassAfter: // fallthrough
			default:
			{
				// Do a pass with stretch only
				for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
				{
					constexpr bool bDampingBefore = false;
					constexpr bool bSingleLambda = false;
					constexpr bool bSeparateStretch = true;
					constexpr bool bDampingAfter = false;
					const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? StiffnessWarp[ConstraintIndex] : StiffnessNoMap;
					const FSolverReal ExpStiffnessWeftValue = StiffnessWeftHasWeightMap ? StiffnessWeft[ConstraintIndex] : StiffnessWeftNoMap;
					const FSolverReal ExpStiffnessBiasValue = StiffnessBiasHasWeightMap ? StiffnessBias[ConstraintIndex] : StiffnessBiasNoMap;
					const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
					ApplyHelper<bDampingBefore, bSingleLambda, bSeparateStretch, bDampingAfter>(Particles, Dt, ConstraintIndex, FSolverVec3(ExpStiffnessWeftValue, ExpStiffnessValue, ExpStiffnessBiasValue), DampingRatioValue);
				}
			}
			break;
			case (int32)EXPBDSplitDampingMode::SingleLambda:
			{
				// Do a pass with combined stretch and damping with a single lambda
				for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
				{
					constexpr bool bDampingBefore = false;
					constexpr bool bSingleLambda = true;
					constexpr bool bSeparateStretch = false;
					constexpr bool bDampingAfter = false;
					const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? StiffnessWarp[ConstraintIndex] : StiffnessNoMap;
					const FSolverReal ExpStiffnessWeftValue = StiffnessWeftHasWeightMap ? StiffnessWeft[ConstraintIndex] : StiffnessWeftNoMap;
					const FSolverReal ExpStiffnessBiasValue = StiffnessBiasHasWeightMap ? StiffnessBias[ConstraintIndex] : StiffnessBiasNoMap;
					const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
					ApplyHelper<bDampingBefore, bSingleLambda, bSeparateStretch, bDampingAfter>(Particles, Dt, ConstraintIndex, FSolverVec3(ExpStiffnessWeftValue, ExpStiffnessValue, ExpStiffnessBiasValue), DampingRatioValue);
				}
			}
			break;
			case (int32)EXPBDSplitDampingMode::InterleavedBefore:
			{
				// Do a pass with damping before stretch
				for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
				{
					constexpr bool bDampingBefore = true;
					constexpr bool bSingleLambda = false;
					constexpr bool bSeparateStretch = true;
					constexpr bool bDampingAfter = false;
					const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? StiffnessWarp[ConstraintIndex] : StiffnessNoMap;
					const FSolverReal ExpStiffnessWeftValue = StiffnessWeftHasWeightMap ? StiffnessWeft[ConstraintIndex] : StiffnessWeftNoMap;
					const FSolverReal ExpStiffnessBiasValue = StiffnessBiasHasWeightMap ? StiffnessBias[ConstraintIndex] : StiffnessBiasNoMap;
					const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
					ApplyHelper<bDampingBefore, bSingleLambda, bSeparateStretch, bDampingAfter>(Particles, Dt, ConstraintIndex, FSolverVec3(ExpStiffnessWeftValue, ExpStiffnessValue, ExpStiffnessBiasValue), DampingRatioValue);
				}
			}
			break;
			case (int32)EXPBDSplitDampingMode::InterleavedAfter:
			{
				// Do a pass with damping after stretch
				for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
				{
					constexpr bool bDampingBefore = false;
					constexpr bool bSingleLambda = false;
					constexpr bool bSeparateStretch = true;
					constexpr bool bDampingAfter = true;
					const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? StiffnessWarp[ConstraintIndex] : StiffnessNoMap;
					const FSolverReal ExpStiffnessWeftValue = StiffnessWeftHasWeightMap ? StiffnessWeft[ConstraintIndex] : StiffnessWeftNoMap;
					const FSolverReal ExpStiffnessBiasValue = StiffnessBiasHasWeightMap ? StiffnessBias[ConstraintIndex] : StiffnessBiasNoMap;
					const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
					ApplyHelper<bDampingBefore, bSingleLambda, bSeparateStretch, bDampingAfter>(Particles, Dt, ConstraintIndex, FSolverVec3(ExpStiffnessWeftValue, ExpStiffnessValue, ExpStiffnessBiasValue), DampingRatioValue);
				}
			}
			break;
			}
			if (Chaos_XPBDSpring_SplitDampingMode == (int32)EXPBDSplitDampingMode::TwoPassAfter)
			{
				for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
				{
					constexpr bool bDampingBefore = false;
					constexpr bool bSingleLambda = false;
					constexpr bool bSeparateStretch = false;
					constexpr bool bDampingAfter = true;
					const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? StiffnessWarp[ConstraintIndex] : StiffnessNoMap;
					const FSolverReal ExpStiffnessWeftValue = StiffnessWeftHasWeightMap ? StiffnessWeft[ConstraintIndex] : StiffnessWeftNoMap;
					const FSolverReal ExpStiffnessBiasValue = StiffnessBiasHasWeightMap ? StiffnessBias[ConstraintIndex] : StiffnessBiasNoMap;
					const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
					ApplyHelper<bDampingBefore, bSingleLambda, bSeparateStretch, bDampingAfter>(Particles, Dt, ConstraintIndex, FSolverVec3(ExpStiffnessWeftValue, ExpStiffnessValue, ExpStiffnessBiasValue), DampingRatioValue);
				}
			}
		}
		else
		{
			// No damping. 
			for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
			{
				constexpr bool bDampingBefore = false;
				constexpr bool bSingleLambda = false;
				constexpr bool bSeparateStretch = true;
				constexpr bool bDampingAfter = false;
				const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? StiffnessWarp[ConstraintIndex] : StiffnessNoMap;
				const FSolverReal ExpStiffnessWeftValue = StiffnessWeftHasWeightMap ? StiffnessWeft[ConstraintIndex] : StiffnessWeftNoMap;
				const FSolverReal ExpStiffnessBiasValue = StiffnessBiasHasWeightMap ? StiffnessBias[ConstraintIndex] : StiffnessBiasNoMap;
				ApplyHelper<bDampingBefore, bSingleLambda, bSeparateStretch, bDampingAfter>(Particles, Dt, ConstraintIndex, FSolverVec3(ExpStiffnessWeftValue, ExpStiffnessValue, ExpStiffnessBiasValue), (FSolverReal)0.);

			}
		}
	}
}
template CHAOS_API void FXPBDAnisotropicAxialSpringConstraints::Apply(FSolverParticles& Particles, const FSolverReal Dt) const;
template CHAOS_API void FXPBDAnisotropicAxialSpringConstraints::Apply(FSolverParticlesRange& Particles, const FSolverReal Dt) const;

void FXPBDAnisotropicAxialSpringConstraints::UpdateLinearSystem(const FSolverParticlesRange& Particles, const FSolverReal Dt, FEvolutionLinearSystem& LinearSystem) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FXPBDAnisotropicAxialSpringConstraints_UpdateLinearSystem);
	LinearSystem.ReserveForParallelAdd(Constraints.Num() * 3, Constraints.Num() * 2);

	const bool StiffnessHasWeightMap = StiffnessWarp.HasWeightMap();
	const bool StiffnessWeftHasWeightMap = StiffnessWeft.HasWeightMap();
	const bool StiffnessBiasHasWeightMap = StiffnessBias.HasWeightMap();
	const bool DampingHasWeightMap = DampingRatio.HasWeightMap();
	if ((ConstraintsPerColorStartIndex.Num() > 1) && (Constraints.Num() > Chaos_XPBDSpring_ParallelConstraintCount))
	{
		const int32 ConstraintColorNum = ConstraintsPerColorStartIndex.Num() - 1;
		if (!StiffnessHasWeightMap && !StiffnessWeftHasWeightMap && !StiffnessBiasHasWeightMap && !DampingHasWeightMap)
		{
			const FSolverVec3 ExpStiffnessValue((FSolverReal)StiffnessWeft, (FSolverReal)StiffnessWarp, (FSolverReal)StiffnessBias);
			const FSolverReal DampingRatioValue = (FSolverReal)DampingRatio;
			if (ExpStiffnessValue.Max() < MinStiffness)
			{
				return;
			}
			for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
			{
				const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
				const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
				PhysicsParallelFor(ColorSize, [this, &Particles, Dt, &LinearSystem, ColorStart, ExpStiffnessValue, DampingRatioValue](const int32 Index)
				{
					const int32 ConstraintIndex = ColorStart + Index;
					const FSolverVec3& WarpWeftBiasBaseMultiplier = WarpWeftBiasBaseMultipliers[ConstraintIndex];
					const FSolverReal FinalStiffnessValue = WarpWeftBiasBaseMultiplier.Dot(ExpStiffnessValue);
					Spring::UpdateAxialSpringLinearSystem(Particles, Dt, Constraints[ConstraintIndex], Barys[ConstraintIndex], Dists[ConstraintIndex], FinalStiffnessValue, MinStiffness, DampingRatioValue, LinearSystem);
				});
			}
		}
		else // Has weight maps
		{
			const FSolverReal StiffnessNoMap = (FSolverReal)StiffnessWarp;
			const FSolverReal StiffnessWeftNoMap = (FSolverReal)StiffnessWeft;
			const FSolverReal StiffnessBiasNoMap = (FSolverReal)StiffnessBias;
			const FSolverReal DampingNoMap = (FSolverReal)DampingRatio;
			for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
			{
				const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
				const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
				PhysicsParallelFor(ColorSize, [this, &Particles, Dt, &LinearSystem, ColorStart, StiffnessHasWeightMap, StiffnessNoMap, StiffnessWeftHasWeightMap, StiffnessWeftNoMap, StiffnessBiasHasWeightMap,
					StiffnessBiasNoMap, DampingHasWeightMap, DampingNoMap](const int32 Index)
				{
					const int32 ConstraintIndex = ColorStart + Index;
					const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? StiffnessWarp[ConstraintIndex] : StiffnessNoMap;
					const FSolverReal ExpStiffnessWeftValue = StiffnessWeftHasWeightMap ? StiffnessWeft[ConstraintIndex] : StiffnessWeftNoMap;
					const FSolverReal ExpStiffnessBiasValue = StiffnessBiasHasWeightMap ? StiffnessBias[ConstraintIndex] : StiffnessBiasNoMap;
					const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
					const FSolverVec3& WarpWeftBiasBaseMultiplier = WarpWeftBiasBaseMultipliers[ConstraintIndex];
					const FSolverReal FinalStiffnessValue = WarpWeftBiasBaseMultiplier.Dot(FSolverVec3(ExpStiffnessWeftValue, ExpStiffnessValue, ExpStiffnessBiasValue));
					Spring::UpdateAxialSpringLinearSystem(Particles, Dt, Constraints[ConstraintIndex], Barys[ConstraintIndex], Dists[ConstraintIndex], FinalStiffnessValue, MinStiffness, DampingRatioValue, LinearSystem);
				});
			}
		}
	}
	else
	{
		if (!StiffnessHasWeightMap && !StiffnessWeftHasWeightMap && !StiffnessBiasHasWeightMap && !DampingHasWeightMap)
		{
			const FSolverVec3 ExpStiffnessValue((FSolverReal)StiffnessWeft, (FSolverReal)StiffnessWarp, (FSolverReal)StiffnessBias);
			const FSolverReal DampingRatioValue = (FSolverReal)DampingRatio;
			if (ExpStiffnessValue.Max() < MinStiffness)
			{
				return;
			}
			for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
			{
				const FSolverVec3& WarpWeftBiasBaseMultiplier = WarpWeftBiasBaseMultipliers[ConstraintIndex];
				const FSolverReal FinalStiffnessValue = WarpWeftBiasBaseMultiplier.Dot(ExpStiffnessValue);
				Spring::UpdateAxialSpringLinearSystem(Particles, Dt, Constraints[ConstraintIndex], Barys[ConstraintIndex], Dists[ConstraintIndex], FinalStiffnessValue, MinStiffness, DampingRatioValue, LinearSystem);
			}
		}
		else
		{
			const FSolverReal StiffnessNoMap = (FSolverReal)StiffnessWarp;
			const FSolverReal StiffnessWeftNoMap = (FSolverReal)StiffnessWeft;
			const FSolverReal StiffnessBiasNoMap = (FSolverReal)StiffnessBias;
			const FSolverReal DampingNoMap = (FSolverReal)DampingRatio;
			for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
			{
				const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? StiffnessWarp[ConstraintIndex] : StiffnessNoMap;
				const FSolverReal ExpStiffnessWeftValue = StiffnessWeftHasWeightMap ? StiffnessWeft[ConstraintIndex] : StiffnessWeftNoMap;
				const FSolverReal ExpStiffnessBiasValue = StiffnessBiasHasWeightMap ? StiffnessBias[ConstraintIndex] : StiffnessBiasNoMap;
				const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
				const FSolverVec3& WarpWeftBiasBaseMultiplier = WarpWeftBiasBaseMultipliers[ConstraintIndex];
				const FSolverReal FinalStiffnessValue = WarpWeftBiasBaseMultiplier.Dot(FSolverVec3(ExpStiffnessWeftValue, ExpStiffnessValue, ExpStiffnessBiasValue));
				Spring::UpdateAxialSpringLinearSystem(Particles, Dt, Constraints[ConstraintIndex], Barys[ConstraintIndex], Dists[ConstraintIndex], FinalStiffnessValue, MinStiffness, DampingRatioValue, LinearSystem);
			}
		}
	}
}

void FXPBDAnisotropicSpringConstraints::SetProperties(
	const FCollectionPropertyConstFacade& PropertyCollection,
	const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps)
{
	if (IsXPBDAnisoSpringStiffnessWarpMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedValue(GetWeightedFloatXPBDAnisoSpringStiffnessWarp(PropertyCollection));
		if (IsXPBDAnisoSpringStiffnessWarpStringDirty(PropertyCollection))
		{
			const FString& WeightMapName = GetXPBDAnisoSpringStiffnessWarpString(PropertyCollection);
			EdgeConstraints.StiffnessWarp = FPBDFlatWeightMap(
				WeightedValue.ClampAxes(0, FXPBDAnisotropicEdgeSpringConstraints::MaxStiffness),
				WeightMaps.FindRef(WeightMapName),
				TConstArrayView<TVec2<int32>>(EdgeConstraints.Constraints),
				EdgeConstraints.ParticleOffset,
				EdgeConstraints.ParticleCount);
			AxialConstraints.StiffnessWarp = FPBDFlatWeightMap(
				WeightedValue.ClampAxes(0, FXPBDAnisotropicAxialSpringConstraints::MaxStiffness),
				WeightMaps.FindRef(WeightMapName),
				TConstArrayView<TVec3<int32>>(AxialConstraints.Constraints),
				AxialConstraints.ParticleOffset,
				AxialConstraints.ParticleCount);
		}
		else
		{
			EdgeConstraints.StiffnessWarp.SetWeightedValue(WeightedValue.ClampAxes(0, FXPBDAnisotropicEdgeSpringConstraints::MaxStiffness));
			AxialConstraints.StiffnessWarp.SetWeightedValue(WeightedValue.ClampAxes(0, FXPBDAnisotropicAxialSpringConstraints::MaxStiffness));
		}
	}
	if (IsXPBDAnisoSpringStiffnessWeftMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedValue(GetWeightedFloatXPBDAnisoSpringStiffnessWeft(PropertyCollection));
		if (IsXPBDAnisoSpringStiffnessWeftStringDirty(PropertyCollection))
		{
			const FString& WeightMapName = GetXPBDAnisoSpringStiffnessWeftString(PropertyCollection);
			EdgeConstraints.StiffnessWeft = FPBDFlatWeightMap(
				WeightedValue.ClampAxes(0, FXPBDAnisotropicEdgeSpringConstraints::MaxStiffness),
				WeightMaps.FindRef(WeightMapName),
				TConstArrayView<TVec2<int32>>(EdgeConstraints.Constraints),
				EdgeConstraints.ParticleOffset,
				EdgeConstraints.ParticleCount);
			AxialConstraints.StiffnessWeft = FPBDFlatWeightMap(
				WeightedValue.ClampAxes(0, FXPBDAnisotropicAxialSpringConstraints::MaxStiffness),
				WeightMaps.FindRef(WeightMapName),
				TConstArrayView<TVec3<int32>>(AxialConstraints.Constraints),
				AxialConstraints.ParticleOffset,
				AxialConstraints.ParticleCount);
		}
		else
		{
			EdgeConstraints.StiffnessWeft.SetWeightedValue(WeightedValue.ClampAxes(0, FXPBDAnisotropicEdgeSpringConstraints::MaxStiffness));
			AxialConstraints.StiffnessWeft.SetWeightedValue(WeightedValue.ClampAxes(0, FXPBDAnisotropicAxialSpringConstraints::MaxStiffness));
		}
	}
	if (IsXPBDAnisoSpringStiffnessBiasMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedValue(GetWeightedFloatXPBDAnisoSpringStiffnessBias(PropertyCollection));
		if (IsXPBDAnisoSpringStiffnessBiasStringDirty(PropertyCollection))
		{
			const FString& WeightMapName = GetXPBDAnisoSpringStiffnessBiasString(PropertyCollection);
			EdgeConstraints.StiffnessBias = FPBDFlatWeightMap(
				WeightedValue.ClampAxes(0, FXPBDAnisotropicEdgeSpringConstraints::MaxStiffness),
				WeightMaps.FindRef(WeightMapName),
				TConstArrayView<TVec2<int32>>(EdgeConstraints.Constraints),
				EdgeConstraints.ParticleOffset,
				EdgeConstraints.ParticleCount);
			AxialConstraints.StiffnessBias = FPBDFlatWeightMap(
				WeightedValue.ClampAxes(0, FXPBDAnisotropicAxialSpringConstraints::MaxStiffness),
				WeightMaps.FindRef(WeightMapName),
				TConstArrayView<TVec3<int32>>(AxialConstraints.Constraints),
				AxialConstraints.ParticleOffset,
				AxialConstraints.ParticleCount);
		}
		else
		{
			EdgeConstraints.StiffnessBias.SetWeightedValue(WeightedValue.ClampAxes(0, FXPBDAnisotropicEdgeSpringConstraints::MaxStiffness));
			AxialConstraints.StiffnessBias.SetWeightedValue(WeightedValue.ClampAxes(0, FXPBDAnisotropicAxialSpringConstraints::MaxStiffness));
		}
	}
	if (IsXPBDAnisoSpringDampingMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedValue = FSolverVec2(GetWeightedFloatXPBDAnisoSpringDamping(PropertyCollection)).ClampAxes(FXPBDAnisotropicEdgeSpringConstraints::MinDampingRatio, FXPBDAnisotropicEdgeSpringConstraints::MaxDampingRatio);
		if (IsXPBDAnisoSpringDampingStringDirty(PropertyCollection))
		{
			const FString& WeightMapName = GetXPBDAnisoSpringDampingString(PropertyCollection);
			AxialConstraints.DampingRatio = FPBDFlatWeightMap(
				WeightedValue,
				WeightMaps.FindRef(WeightMapName),
				TConstArrayView<TVec3<int32>>(AxialConstraints.Constraints),
				AxialConstraints.ParticleOffset,
				AxialConstraints.ParticleCount);
			AxialConstraints.DampingRatio = FPBDFlatWeightMap(
				WeightedValue,
				WeightMaps.FindRef(WeightMapName),
				TConstArrayView<TVec2<int32>>(EdgeConstraints.Constraints),
				EdgeConstraints.ParticleOffset,
				EdgeConstraints.ParticleCount);
		}
		else
		{
			EdgeConstraints.DampingRatio.SetWeightedValue(WeightedValue);
			AxialConstraints.DampingRatio.SetWeightedValue(WeightedValue);
		}
	}
	if (IsXPBDAnisoSpringWarpScaleMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedValue = FSolverVec2(GetWeightedFloatXPBDAnisoSpringWarpScale(PropertyCollection)).ClampAxes(FXPBDAnisotropicEdgeSpringConstraints::MinWarpWeftScale, FXPBDAnisotropicEdgeSpringConstraints::MaxWarpWeftScale);
		if (IsXPBDAnisoSpringWarpScaleStringDirty(PropertyCollection))
		{
			const FString& WeightMapName = GetXPBDAnisoSpringWarpScaleString(PropertyCollection);
			EdgeConstraints.WarpScale = FPBDWeightMap(
				WeightedValue,
				WeightMaps.FindRef(WeightMapName),
				TConstArrayView<TVec2<int32>>(EdgeConstraints.Constraints),
				EdgeConstraints.ParticleOffset,
				EdgeConstraints.ParticleCount);
			AxialConstraints.WarpScale = FPBDWeightMap(
				WeightedValue,
				WeightMaps.FindRef(WeightMapName),
				TConstArrayView<TVec3<int32>>(AxialConstraints.Constraints),
				AxialConstraints.ParticleOffset,
				AxialConstraints.ParticleCount);
		}
		else
		{
			EdgeConstraints.WarpScale.SetWeightedValue(WeightedValue);
			AxialConstraints.WarpScale.SetWeightedValue(WeightedValue);
		}
	}
	if (IsXPBDAnisoSpringWeftScaleMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedValue = FSolverVec2(GetWeightedFloatXPBDAnisoSpringWeftScale(PropertyCollection)).ClampAxes(FXPBDAnisotropicEdgeSpringConstraints::MinWarpWeftScale, FXPBDAnisotropicEdgeSpringConstraints::MaxWarpWeftScale);
		if (IsXPBDAnisoSpringWeftScaleStringDirty(PropertyCollection))
		{
			const FString& WeightMapName = GetXPBDAnisoSpringWeftScaleString(PropertyCollection);
			EdgeConstraints.WeftScale = FPBDWeightMap(
				WeightedValue,
				WeightMaps.FindRef(WeightMapName),
				TConstArrayView<TVec2<int32>>(EdgeConstraints.Constraints),
				EdgeConstraints.ParticleOffset,
				EdgeConstraints.ParticleCount);
			AxialConstraints.WeftScale = FPBDWeightMap(
				WeightedValue,
				WeightMaps.FindRef(WeightMapName),
				TConstArrayView<TVec3<int32>>(AxialConstraints.Constraints),
				AxialConstraints.ParticleOffset,
				AxialConstraints.ParticleCount);
		}
		else
		{
			EdgeConstraints.WeftScale.SetWeightedValue(WeightedValue);
			AxialConstraints.WeftScale.SetWeightedValue(WeightedValue);
		}
	}
}
}  // End namespace Chaos::Softs
