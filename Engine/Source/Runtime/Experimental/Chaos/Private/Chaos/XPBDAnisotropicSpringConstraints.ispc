// Copyright Epic Games, Inc. All Rights Reserved.

#define EXPLICIT_VECTOR4 1

#include "Math/Vector.isph"
#include "Chaos/PBDSofts.isph"
#include "Chaos/SoftsSpring.isph"

export void ApplyXPBDAnisoSpringConstraints(uniform FVector4f PandInvM[],
									const uniform FIntVector2 Constraints[],
									const uniform float Dists[],
									const uniform FVector3f WarpWeftBiasBaseMultiplers[],
									uniform float Lambdas[],
									const uniform float Dt,
									const uniform FVector3f& Stiffness,
									const uniform int32 NumConstraints)
{
	foreach(Index = 0 ... NumConstraints)
	{
		const varying FIntVector2 Constraint = VectorLoad(&Constraints[extract(Index,0)]);
		const varying int32 Index1 = Constraint.V[0];
		const varying int32 Index2 = Constraint.V[1];

		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[Index1]);
		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[Index2]);

		varying FVector3f P1, P2;
		varying float InvM1, InvM2;
		UnzipPandInvM(PandInvM1, P1, InvM1);
		UnzipPandInvM(PandInvM2, P2, InvM2);
		
		const varying float Dist = Dists[Index];
		const varying float Lambda = Lambdas[Index];
		const varying FVector3f WarpWeftBiasBaseMultiplier = VectorLoad(&WarpWeftBiasBaseMultiplers[extract(Index,0)]);
		const varying float StiffnessValue = VectorDot(WarpWeftBiasBaseMultiplier, Stiffness);

		varying float DLambda;
		const varying FVector3f Delta = GetXPBDSpringDelta(P1, P2, InvM1, InvM2, Dt, Dist, Lambda, StiffnessValue, DLambda);

		Lambdas[Index] = Lambda + DLambda;

		VectorScatter(&PandInvM[Index1], SetVector4( P1 + (InvM1 * Delta), InvM1 ));
		VectorScatter(&PandInvM[Index2], SetVector4( P2 - (InvM2 * Delta), InvM2 ));
	}
}

export void ApplyXPBDAnisoSpringConstraintsWithDamping(uniform FVector4f PandInvM[],
												const uniform FVector3f X[],
												const uniform FIntVector2 Constraints[],
												const uniform float Dists[],
												const uniform FVector3f WarpWeftBiasBaseMultiplers[],
												uniform float Lambdas[],
												const uniform float Dt,
												const uniform FVector3f& Stiffness,
												const uniform float DampingRatio,
												const uniform int32 NumConstraints)
{
	foreach(Index = 0 ... NumConstraints)
	{
		const varying FIntVector2 Constraint = VectorLoad(&Constraints[extract(Index,0)]);
		const varying int32 Index1 = Constraint.V[0];
		const varying int32 Index2 = Constraint.V[1];

		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[Index1]);
		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[Index2]);

		varying FVector3f P1, P2;
		varying float InvM1, InvM2;
		UnzipPandInvM(PandInvM1, P1, InvM1);
		UnzipPandInvM(PandInvM2, P2, InvM2);

		const varying FVector3f X1 = VectorGather(&X[Index1]);
		const varying FVector3f X2 = VectorGather(&X[Index2]);

		const varying float Dist = Dists[Index];
		const varying float Lambda = Lambdas[Index];
		const varying FVector3f WarpWeftBiasBaseMultiplier = VectorLoad(&WarpWeftBiasBaseMultiplers[extract(Index,0)]);
		const varying float StiffnessValue = VectorDot(WarpWeftBiasBaseMultiplier, Stiffness);

		varying float DLambda;
		const FVector3f Delta = GetXPBDSpringDeltaWithDamping(P1, P2, X1, X2, InvM1, InvM2, Dt, Dist, Lambda, StiffnessValue, DampingRatio, DLambda);

		Lambdas[Index] = Lambda + DLambda;
		
		VectorScatter(&PandInvM[Index1], SetVector4( P1 + (InvM1 * Delta), InvM1 ));
		VectorScatter(&PandInvM[Index2], SetVector4( P2 - (InvM2 * Delta), InvM2 ));
	}
}

export void ApplyXPBDAnisoSpringDampingConstraints(uniform FVector4f PandInvM[],
												const uniform FVector3f X[],
												const uniform FIntVector2 Constraints[],
												const uniform float Dists[],
												const uniform FVector3f WarpWeftBiasBaseMultiplers[],
												uniform float Lambdas[],
												const uniform float Dt,
												const uniform FVector3f& Stiffness,
												const uniform float DampingRatio,
												const uniform int32 NumConstraints)
{
	foreach(Index = 0 ... NumConstraints)
	{
		const varying FIntVector2 Constraint = VectorLoad(&Constraints[extract(Index,0)]);
		const varying int32 Index1 = Constraint.V[0];
		const varying int32 Index2 = Constraint.V[1];

		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[Index1]);
		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[Index2]);

		varying FVector3f P1, P2;
		varying float InvM1, InvM2;
		UnzipPandInvM(PandInvM1, P1, InvM1);
		UnzipPandInvM(PandInvM2, P2, InvM2);

		const varying FVector3f X1 = VectorGather(&X[Index1]);
		const varying FVector3f X2 = VectorGather(&X[Index2]);

		const varying float Dist = Dists[Index];
		const varying float Lambda = Lambdas[Index];
		const varying FVector3f WarpWeftBiasBaseMultiplier = VectorLoad(&WarpWeftBiasBaseMultiplers[extract(Index,0)]);
		const varying float StiffnessValue = VectorDot(WarpWeftBiasBaseMultiplier, Stiffness);

		varying float DLambda;
		const FVector3f Delta = GetXPBDSpringDampingDelta(P1, P2, X1, X2, InvM1, InvM2, Dt, Dist, Lambda, StiffnessValue, DampingRatio, DLambda);

		Lambdas[Index] = Lambda + DLambda;
		
		VectorScatter(&PandInvM[Index1], SetVector4( P1 + (InvM1 * Delta), InvM1 ));
		VectorScatter(&PandInvM[Index2], SetVector4( P2 - (InvM2 * Delta), InvM2 ));
	}
}

export void ApplyXPBDAnisoSpringConstraintsWithWeightMaps(uniform FVector4f PandInvM[],
									const uniform FIntVector2 Constraints[],
									const uniform float Dists[],
									const uniform FVector3f WarpWeftBiasBaseMultiplers[],
									uniform float Lambdas[],
									const uniform float Dt,
									const uniform bool StiffnessHasMap,
									const uniform FVector2f& StiffnessOffsetRange,
									const uniform float StiffnessMapValues[],
									const uniform bool StiffnessWeftHasMap,
									const uniform FVector2f& StiffnessWeftOffsetRange,
									const uniform float StiffnessWeftMapValues[],
									const uniform bool StiffnessBiasHasMap,
									const uniform FVector2f& StiffnessBiasOffsetRange,
									const uniform float StiffnessBiasMapValues[],
									const uniform int32 NumConstraints)
{

	foreach(Index = 0 ... NumConstraints)
	{
		const varying FIntVector2 Constraint = VectorLoad(&Constraints[extract(Index,0)]);
		const varying int32 Index1 = Constraint.V[0];
		const varying int32 Index2 = Constraint.V[1];

		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[Index1]);
		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[Index2]);

		varying FVector3f P1, P2;
		varying float InvM1, InvM2;
		UnzipPandInvM(PandInvM1, P1, InvM1);
		UnzipPandInvM(PandInvM2, P2, InvM2);

		const varying float Dist = Dists[Index];
		const varying float Lambda = Lambdas[Index];

		const varying float Stiffness = StiffnessHasMap ? StiffnessOffsetRange.V[0] + StiffnessMapValues[Index]*StiffnessOffsetRange.V[1] : StiffnessOffsetRange.V[0];
		const varying float StiffnessWeft = StiffnessWeftHasMap ? StiffnessWeftOffsetRange.V[0] + StiffnessWeftMapValues[Index]*StiffnessWeftOffsetRange.V[1] : StiffnessWeftOffsetRange.V[0];
		const varying float StiffnessBias = StiffnessBiasHasMap ? StiffnessBiasOffsetRange.V[0] + StiffnessBiasMapValues[Index]*StiffnessBiasOffsetRange.V[1] : StiffnessBiasOffsetRange.V[0];
		
		const varying FVector3f StiffnessVector = {{StiffnessWeft, Stiffness, StiffnessBias}};
		const varying FVector3f WarpWeftBiasBaseMultiplier = VectorLoad(&WarpWeftBiasBaseMultiplers[extract(Index,0)]);
		const varying float StiffnessValue = VectorDot(WarpWeftBiasBaseMultiplier, StiffnessVector);

		varying float DLambda;
		const varying FVector3f Delta = GetXPBDSpringDelta(P1, P2, InvM1, InvM2, Dt, Dist, Lambda, StiffnessValue, DLambda);

		Lambdas[Index] = Lambda + DLambda;
		VectorScatter(&PandInvM[Index1], SetVector4( P1 + (InvM1 * Delta), InvM1 ));
		VectorScatter(&PandInvM[Index2], SetVector4( P2 - (InvM2 * Delta), InvM2 ));
	}
}

export void ApplyXPBDAnisoSpringConstraintsWithDampingAndWeightMaps(uniform FVector4f PandInvM[],
															const uniform FVector3f X[],
															const uniform FIntVector2 Constraints[],
															const uniform float Dists[],
															const uniform FVector3f WarpWeftBiasBaseMultiplers[],
															uniform float Lambdas[],
															const uniform float Dt,
															const uniform bool StiffnessHasMap,
															const uniform FVector2f& StiffnessOffsetRange,
															const uniform float StiffnessMapValues[],
															const uniform bool StiffnessWeftHasMap,
															const uniform FVector2f& StiffnessWeftOffsetRange,
															const uniform float StiffnessWeftMapValues[],
															const uniform bool StiffnessBiasHasMap,
															const uniform FVector2f& StiffnessBiasOffsetRange,
															const uniform float StiffnessBiasMapValues[],
															const uniform bool DampingHasMap,
															const uniform FVector2f& DampingOffsetRange,
															const uniform float DampingMapValues[],
															const uniform int32 NumConstraints)
{

	foreach(Index = 0 ... NumConstraints)
	{
		const varying FIntVector2 Constraint = VectorLoad(&Constraints[extract(Index,0)]);
		const varying int32 Index1 = Constraint.V[0];
		const varying int32 Index2 = Constraint.V[1];

		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[Index1]);
		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[Index2]);

		varying FVector3f P1, P2;
		varying float InvM1, InvM2;
		UnzipPandInvM(PandInvM1, P1, InvM1);
		UnzipPandInvM(PandInvM2, P2, InvM2);

		const varying FVector3f X1 = VectorGather(&X[Index1]);
		const varying FVector3f X2 = VectorGather(&X[Index2]);

		const varying float Dist = Dists[Index];
		const varying float Lambda = Lambdas[Index];
		
		const varying float Stiffness = StiffnessHasMap ? StiffnessOffsetRange.V[0] + StiffnessMapValues[Index]*StiffnessOffsetRange.V[1] : StiffnessOffsetRange.V[0];
		const varying float StiffnessWeft = StiffnessWeftHasMap ? StiffnessWeftOffsetRange.V[0] + StiffnessWeftMapValues[Index]*StiffnessWeftOffsetRange.V[1] : StiffnessWeftOffsetRange.V[0];
		const varying float StiffnessBias = StiffnessBiasHasMap ? StiffnessBiasOffsetRange.V[0] + StiffnessBiasMapValues[Index]*StiffnessBiasOffsetRange.V[1] : StiffnessBiasOffsetRange.V[0];

		const varying FVector3f StiffnessVector = {{StiffnessWeft, Stiffness, StiffnessBias}};
		const varying FVector3f WarpWeftBiasBaseMultiplier = VectorLoad(&WarpWeftBiasBaseMultiplers[extract(Index,0)]);
		const varying float StiffnessValue = VectorDot(WarpWeftBiasBaseMultiplier, StiffnessVector);

		const varying float DampingRatio = DampingHasMap ? DampingOffsetRange.V[0] + DampingMapValues[Index]*DampingOffsetRange.V[1] : DampingOffsetRange.V[0];

		varying float DLambda;
		const FVector3f Delta = GetXPBDSpringDeltaWithDamping(P1, P2, X1, X2, InvM1, InvM2, Dt, Dist, Lambda, StiffnessValue, DampingRatio, DLambda);

		Lambdas[Index] = Lambda + DLambda;

		VectorScatter(&PandInvM[Index1], SetVector4( P1 + (InvM1 * Delta), InvM1 ));
		VectorScatter(&PandInvM[Index2], SetVector4( P2 - (InvM2 * Delta), InvM2 ));
	}
}

export void ApplyXPBDAnisoSpringDampingConstraintsWithWeightMaps(uniform FVector4f PandInvM[],
															const uniform FVector3f X[],
															const uniform FIntVector2 Constraints[],
															const uniform float Dists[],
															const uniform FVector3f WarpWeftBiasBaseMultiplers[],
															uniform float Lambdas[],
															const uniform float Dt,
															const uniform bool StiffnessHasMap,
															const uniform FVector2f& StiffnessOffsetRange,
															const uniform float StiffnessMapValues[],
															const uniform bool StiffnessWeftHasMap,
															const uniform FVector2f& StiffnessWeftOffsetRange,
															const uniform float StiffnessWeftMapValues[],
															const uniform bool StiffnessBiasHasMap,
															const uniform FVector2f& StiffnessBiasOffsetRange,
															const uniform float StiffnessBiasMapValues[],
															const uniform bool DampingHasMap,
															const uniform FVector2f& DampingOffsetRange,
															const uniform float DampingMapValues[],
															const uniform int32 NumConstraints)
{

	foreach(Index = 0 ... NumConstraints)
	{
		const varying FIntVector2 Constraint = VectorLoad(&Constraints[extract(Index,0)]);
		const varying int32 Index1 = Constraint.V[0];
		const varying int32 Index2 = Constraint.V[1];

		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[Index1]);
		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[Index2]);

		varying FVector3f P1, P2;
		varying float InvM1, InvM2;
		UnzipPandInvM(PandInvM1, P1, InvM1);
		UnzipPandInvM(PandInvM2, P2, InvM2);

		const varying FVector3f X1 = VectorGather(&X[Index1]);
		const varying FVector3f X2 = VectorGather(&X[Index2]);

		const varying float Dist = Dists[Index];
		const varying float Lambda = Lambdas[Index];

		const varying float Stiffness = StiffnessHasMap ? StiffnessOffsetRange.V[0] + StiffnessMapValues[Index]*StiffnessOffsetRange.V[1] : StiffnessOffsetRange.V[0];
		const varying float StiffnessWeft = StiffnessWeftHasMap ? StiffnessWeftOffsetRange.V[0] + StiffnessWeftMapValues[Index]*StiffnessWeftOffsetRange.V[1] : StiffnessWeftOffsetRange.V[0];
		const varying float StiffnessBias = StiffnessBiasHasMap ? StiffnessBiasOffsetRange.V[0] + StiffnessBiasMapValues[Index]*StiffnessBiasOffsetRange.V[1] : StiffnessBiasOffsetRange.V[0];

		const varying FVector3f StiffnessVector = {{StiffnessWeft, Stiffness, StiffnessBias}};
		const varying FVector3f WarpWeftBiasBaseMultiplier = VectorLoad(&WarpWeftBiasBaseMultiplers[extract(Index,0)]);
		const varying float StiffnessValue = VectorDot(WarpWeftBiasBaseMultiplier, StiffnessVector);
		
		const varying float DampingRatio = DampingHasMap ? DampingOffsetRange.V[0] + DampingMapValues[Index]*DampingOffsetRange.V[1] : DampingOffsetRange.V[0];

		varying float DLambda;
		const FVector3f Delta = GetXPBDSpringDampingDelta(P1, P2, X1, X2, InvM1, InvM2, Dt, Dist, Lambda, StiffnessValue, DampingRatio, DLambda);

		Lambdas[Index] = Lambda + DLambda;
		
		VectorScatter(&PandInvM[Index1], SetVector4( P1 + (InvM1 * Delta), InvM1 ));
		VectorScatter(&PandInvM[Index2], SetVector4( P2 - (InvM2 * Delta), InvM2 ));
	}
}

export void ApplyXPBDAnisoAxialSpringConstraints(uniform FVector4f PandInvM[],
									const uniform FIntVector Constraints[],
									const uniform float Barys[],
									const uniform float Dists[],
									const uniform FVector3f WarpWeftBiasBaseMultiplers[],
									uniform float Lambdas[],
									const uniform float Dt,
									const uniform FVector3f& Stiffness,
									const uniform int32 NumConstraints)
{
	foreach(Index = 0 ... NumConstraints)
	{
		const varying FIntVector Constraint = VectorLoad(&Constraints[extract(Index,0)]);
		const varying int32 Index1 = Constraint.V[0];
		const varying int32 Index2 = Constraint.V[1];
		const varying int32 Index3 = Constraint.V[2];

		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[Index1]);
		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[Index2]);
		const varying FVector4f PandInvM3 = VectorGather(&PandInvM[Index3]);

		varying FVector3f P1, P2, P3;
		varying float InvM1, InvM2, InvM3;
		UnzipPandInvM(PandInvM1, P1, InvM1);
		UnzipPandInvM(PandInvM2, P2, InvM2);
		UnzipPandInvM(PandInvM3, P3, InvM3);
		
		const varying float Bary = Barys[Index];
		const varying float Dist = Dists[Index];
		const varying float Lambda = Lambdas[Index];
		const varying FVector3f WarpWeftBiasBaseMultiplier = VectorLoad(&WarpWeftBiasBaseMultiplers[extract(Index,0)]);
		const varying float StiffnessValue = VectorDot(WarpWeftBiasBaseMultiplier, Stiffness);

		varying float DLambda;
		const varying FVector3f Delta = GetXPBDAxialSpringDelta(P1, P2, P3, InvM1, InvM2, InvM3, Dt, Bary, Dist, Lambda, StiffnessValue, DLambda);

		Lambdas[Index] = Lambda + DLambda;

		VectorScatter(&PandInvM[Index1], SetVector4( P1 + (InvM1 * Delta), InvM1 ));
		VectorScatter(&PandInvM[Index2], SetVector4( P2 - (InvM2 * Bary * Delta), InvM2 ));
		VectorScatter(&PandInvM[Index3], SetVector4( P3 - (InvM3 * (1.f - Bary) * Delta), InvM3 ));
	}
}

export void ApplyXPBDAnisoAxialSpringConstraintsWithDamping(uniform FVector4f PandInvM[],
												const uniform FVector3f X[],
												const uniform FIntVector Constraints[],
												const uniform float Barys[],
												const uniform float Dists[],
												const uniform FVector3f WarpWeftBiasBaseMultiplers[],
												uniform float Lambdas[],
												const uniform float Dt,
												const uniform FVector3f& Stiffness,
												const uniform float DampingRatio,
												const uniform int32 NumConstraints)
{
	foreach(Index = 0 ... NumConstraints)
	{
		const varying FIntVector Constraint = VectorLoad(&Constraints[extract(Index,0)]);
		const varying int32 Index1 = Constraint.V[0];
		const varying int32 Index2 = Constraint.V[1];
		const varying int32 Index3 = Constraint.V[2];
		
		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[Index1]);
		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[Index2]);
		const varying FVector4f PandInvM3 = VectorGather(&PandInvM[Index3]);

		varying FVector3f P1, P2, P3;
		varying float InvM1, InvM2, InvM3;
		UnzipPandInvM(PandInvM1, P1, InvM1);
		UnzipPandInvM(PandInvM2, P2, InvM2);
		UnzipPandInvM(PandInvM3, P3, InvM3);

		const varying FVector3f X1 = VectorGather(&X[Index1]);
		const varying FVector3f X2 = VectorGather(&X[Index2]);
		const varying FVector3f X3 = VectorGather(&X[Index3]);
		
		const varying float Bary = Barys[Index];
		const varying float Dist = Dists[Index];
		const varying float Lambda = Lambdas[Index];
		const varying FVector3f WarpWeftBiasBaseMultiplier = VectorLoad(&WarpWeftBiasBaseMultiplers[extract(Index,0)]);
		const varying float StiffnessValue = VectorDot(WarpWeftBiasBaseMultiplier, Stiffness);

		varying float DLambda;
		const FVector3f Delta = GetXPBDAxialSpringDeltaWithDamping(P1, P2, P3, X1, X2, X3, InvM1, InvM2, InvM3, Dt, Bary, Dist, Lambda, StiffnessValue, DampingRatio, DLambda);

		Lambdas[Index] = Lambda + DLambda;

		VectorScatter(&PandInvM[Index1], SetVector4( P1 + (InvM1 * Delta), InvM1 ));
		VectorScatter(&PandInvM[Index2], SetVector4( P2 - (InvM2 * Bary * Delta), InvM2 ));
		VectorScatter(&PandInvM[Index3], SetVector4( P3 - (InvM3 * (1.f - Bary) * Delta), InvM3 ));
	}
}

export void ApplyXPBDAnisoAxialSpringDampingConstraints(uniform FVector4f PandInvM[],
												const uniform FVector3f X[],
												const uniform FIntVector Constraints[],
												const uniform float Barys[],
												const uniform float Dists[],
												const uniform FVector3f WarpWeftBiasBaseMultiplers[],
												uniform float Lambdas[],
												const uniform float Dt,
												const uniform FVector3f& Stiffness,
												const uniform float DampingRatio,
												const uniform int32 NumConstraints)
{
	foreach(Index = 0 ... NumConstraints)
	{
		const varying FIntVector Constraint = VectorLoad(&Constraints[extract(Index,0)]);
		const varying int32 Index1 = Constraint.V[0];
		const varying int32 Index2 = Constraint.V[1];
		const varying int32 Index3 = Constraint.V[2];
		
		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[Index1]);
		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[Index2]);
		const varying FVector4f PandInvM3 = VectorGather(&PandInvM[Index3]);

		varying FVector3f P1, P2, P3;
		varying float InvM1, InvM2, InvM3;
		UnzipPandInvM(PandInvM1, P1, InvM1);
		UnzipPandInvM(PandInvM2, P2, InvM2);
		UnzipPandInvM(PandInvM3, P3, InvM3);

		const varying FVector3f X1 = VectorGather(&X[Index1]);
		const varying FVector3f X2 = VectorGather(&X[Index2]);
		const varying FVector3f X3 = VectorGather(&X[Index3]);
		
		const varying float Bary = Barys[Index];
		const varying float Dist = Dists[Index];
		const varying float Lambda = Lambdas[Index];
		const varying FVector3f WarpWeftBiasBaseMultiplier = VectorLoad(&WarpWeftBiasBaseMultiplers[extract(Index,0)]);
		const varying float StiffnessValue = VectorDot(WarpWeftBiasBaseMultiplier, Stiffness);

		varying float DLambda;
		const FVector3f Delta = GetXPBDAxialSpringDampingDelta(P1, P2, P3, X1, X2, X3, InvM1, InvM2, InvM3, Dt, Bary, Dist, Lambda, StiffnessValue, DampingRatio, DLambda);

		Lambdas[Index] = Lambda + DLambda;

		VectorScatter(&PandInvM[Index1], SetVector4( P1 + (InvM1 * Delta), InvM1 ));
		VectorScatter(&PandInvM[Index2], SetVector4( P2 - (InvM2 * Bary * Delta), InvM2 ));
		VectorScatter(&PandInvM[Index3], SetVector4( P3 - (InvM3 * (1.f - Bary) * Delta), InvM3 ));
	}
}

export void ApplyXPBDAnisoAxialSpringConstraintsWithWeightMaps(uniform FVector4f PandInvM[],
									const uniform FIntVector Constraints[],
									const uniform float Barys[],
									const uniform float Dists[],
									const uniform FVector3f WarpWeftBiasBaseMultiplers[],
									uniform float Lambdas[],
									const uniform float Dt,
									const uniform bool StiffnessHasMap,
									const uniform FVector2f& StiffnessOffsetRange,
									const uniform float StiffnessMapValues[],
									const uniform bool StiffnessWeftHasMap,
									const uniform FVector2f& StiffnessWeftOffsetRange,
									const uniform float StiffnessWeftMapValues[],
									const uniform bool StiffnessBiasHasMap,
									const uniform FVector2f& StiffnessBiasOffsetRange,
									const uniform float StiffnessBiasMapValues[],
									const uniform int32 NumConstraints)
{

	foreach(Index = 0 ... NumConstraints)
	{
		const varying FIntVector Constraint = VectorLoad(&Constraints[extract(Index,0)]);
		const varying int32 Index1 = Constraint.V[0];
		const varying int32 Index2 = Constraint.V[1];
		const varying int32 Index3 = Constraint.V[2];
		
		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[Index1]);
		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[Index2]);
		const varying FVector4f PandInvM3 = VectorGather(&PandInvM[Index3]);

		varying FVector3f P1, P2, P3;
		varying float InvM1, InvM2, InvM3;
		UnzipPandInvM(PandInvM1, P1, InvM1);
		UnzipPandInvM(PandInvM2, P2, InvM2);
		UnzipPandInvM(PandInvM3, P3, InvM3);
		
		const varying float Bary = Barys[Index];
		const varying float Dist = Dists[Index];
		const varying float Lambda = Lambdas[Index];
		
		const varying float Stiffness = StiffnessHasMap ? StiffnessOffsetRange.V[0] + StiffnessMapValues[Index]*StiffnessOffsetRange.V[1] : StiffnessOffsetRange.V[0];
		const varying float StiffnessWeft = StiffnessWeftHasMap ? StiffnessWeftOffsetRange.V[0] + StiffnessWeftMapValues[Index]*StiffnessWeftOffsetRange.V[1] : StiffnessWeftOffsetRange.V[0];
		const varying float StiffnessBias = StiffnessBiasHasMap ? StiffnessBiasOffsetRange.V[0] + StiffnessBiasMapValues[Index]*StiffnessBiasOffsetRange.V[1] : StiffnessBiasOffsetRange.V[0];
		
		const varying FVector3f StiffnessVector = {{StiffnessWeft, Stiffness, StiffnessBias}};
		const varying FVector3f WarpWeftBiasBaseMultiplier = VectorLoad(&WarpWeftBiasBaseMultiplers[extract(Index,0)]);
		const varying float StiffnessValue = VectorDot(WarpWeftBiasBaseMultiplier, StiffnessVector);

		varying float DLambda;
		const varying FVector3f Delta = GetXPBDAxialSpringDelta(P1, P2, P3, InvM1, InvM2, InvM3, Dt, Bary, Dist, Lambda, StiffnessValue, DLambda);

		Lambdas[Index] = Lambda + DLambda;
		
		VectorScatter(&PandInvM[Index1], SetVector4( P1 + (InvM1 * Delta), InvM1 ));
		VectorScatter(&PandInvM[Index2], SetVector4( P2 - (InvM2 * Bary * Delta), InvM2 ));
		VectorScatter(&PandInvM[Index3], SetVector4( P3 - (InvM3 * (1.f - Bary) * Delta), InvM3 ));
	}
}

export void ApplyXPBDAnisoAxialSpringConstraintsWithDampingAndWeightMaps(uniform FVector4f PandInvM[],
															const uniform FVector3f X[],
															const uniform FIntVector Constraints[],
															const uniform float Barys[],
															const uniform float Dists[],
															const uniform FVector3f WarpWeftBiasBaseMultiplers[],
															uniform float Lambdas[],
															const uniform float Dt,
															const uniform bool StiffnessHasMap,
															const uniform FVector2f& StiffnessOffsetRange,
															const uniform float StiffnessMapValues[],
															const uniform bool StiffnessWeftHasMap,
															const uniform FVector2f& StiffnessWeftOffsetRange,
															const uniform float StiffnessWeftMapValues[],
															const uniform bool StiffnessBiasHasMap,
															const uniform FVector2f& StiffnessBiasOffsetRange,
															const uniform float StiffnessBiasMapValues[],
															const uniform bool DampingHasMap,
															const uniform FVector2f& DampingOffsetRange,
															const uniform float DampingMapValues[],
															const uniform int32 NumConstraints)
{

	foreach(Index = 0 ... NumConstraints)
	{
		const varying FIntVector Constraint = VectorLoad(&Constraints[extract(Index,0)]);
		const varying int32 Index1 = Constraint.V[0];
		const varying int32 Index2 = Constraint.V[1];
		const varying int32 Index3 = Constraint.V[2];
		
		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[Index1]);
		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[Index2]);
		const varying FVector4f PandInvM3 = VectorGather(&PandInvM[Index3]);

		varying FVector3f P1, P2, P3;
		varying float InvM1, InvM2, InvM3;
		UnzipPandInvM(PandInvM1, P1, InvM1);
		UnzipPandInvM(PandInvM2, P2, InvM2);
		UnzipPandInvM(PandInvM3, P3, InvM3);
		
		const varying FVector3f X1 = VectorGather(&X[Index1]);
		const varying FVector3f X2 = VectorGather(&X[Index2]);
		const varying FVector3f X3 = VectorGather(&X[Index3]);
		
		const varying float Bary = Barys[Index];
		const varying float Dist = Dists[Index];
		const varying float Lambda = Lambdas[Index];		
		
		const varying float Stiffness = StiffnessHasMap ? StiffnessOffsetRange.V[0] + StiffnessMapValues[Index]*StiffnessOffsetRange.V[1] : StiffnessOffsetRange.V[0];
		const varying float StiffnessWeft = StiffnessWeftHasMap ? StiffnessWeftOffsetRange.V[0] + StiffnessWeftMapValues[Index]*StiffnessWeftOffsetRange.V[1] : StiffnessWeftOffsetRange.V[0];
		const varying float StiffnessBias = StiffnessBiasHasMap ? StiffnessBiasOffsetRange.V[0] + StiffnessBiasMapValues[Index]*StiffnessBiasOffsetRange.V[1] : StiffnessBiasOffsetRange.V[0];

		const varying FVector3f StiffnessVector = {{StiffnessWeft, Stiffness, StiffnessBias}};
		const varying FVector3f WarpWeftBiasBaseMultiplier = VectorLoad(&WarpWeftBiasBaseMultiplers[extract(Index,0)]);
		const varying float StiffnessValue = VectorDot(WarpWeftBiasBaseMultiplier, StiffnessVector);
		
		const varying float DampingRatio = DampingHasMap ? DampingOffsetRange.V[0] + DampingMapValues[Index]*DampingOffsetRange.V[1] : DampingOffsetRange.V[0];

		varying float DLambda;
		const FVector3f Delta = GetXPBDAxialSpringDeltaWithDamping(P1, P2, P3, X1, X2, X3, InvM1, InvM2, InvM3, Dt, Bary, Dist, Lambda, StiffnessValue, DampingRatio, DLambda);

		Lambdas[Index] = Lambda + DLambda;
		
		VectorScatter(&PandInvM[Index1], SetVector4( P1 + (InvM1 * Delta), InvM1 ));
		VectorScatter(&PandInvM[Index2], SetVector4( P2 - (InvM2 * Bary * Delta), InvM2 ));
		VectorScatter(&PandInvM[Index3], SetVector4( P3 - (InvM3 * (1.f - Bary) * Delta), InvM3 ));
	}
}

export void ApplyXPBDAnisoAxialSpringDampingConstraintsWithWeightMaps(uniform FVector4f PandInvM[],
															const uniform FVector3f X[],
															const uniform FIntVector Constraints[],
															const uniform float Barys[],
															const uniform float Dists[],
															const uniform FVector3f WarpWeftBiasBaseMultiplers[],
															uniform float Lambdas[],
															const uniform float Dt,
															const uniform bool StiffnessHasMap,
															const uniform FVector2f& StiffnessOffsetRange,
															const uniform float StiffnessMapValues[],
															const uniform bool StiffnessWeftHasMap,
															const uniform FVector2f& StiffnessWeftOffsetRange,
															const uniform float StiffnessWeftMapValues[],
															const uniform bool StiffnessBiasHasMap,
															const uniform FVector2f& StiffnessBiasOffsetRange,
															const uniform float StiffnessBiasMapValues[],
															const uniform bool DampingHasMap,
															const uniform FVector2f& DampingOffsetRange,
															const uniform float DampingMapValues[],
															const uniform int32 NumConstraints)
{

	foreach(Index = 0 ... NumConstraints)
	{
		const varying FIntVector Constraint = VectorLoad(&Constraints[extract(Index,0)]);
		const varying int32 Index1 = Constraint.V[0];
		const varying int32 Index2 = Constraint.V[1];
		const varying int32 Index3 = Constraint.V[2];
		
		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[Index1]);
		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[Index2]);
		const varying FVector4f PandInvM3 = VectorGather(&PandInvM[Index3]);

		varying FVector3f P1, P2, P3;
		varying float InvM1, InvM2, InvM3;
		UnzipPandInvM(PandInvM1, P1, InvM1);
		UnzipPandInvM(PandInvM2, P2, InvM2);
		UnzipPandInvM(PandInvM3, P3, InvM3);
		
		const varying FVector3f X1 = VectorGather(&X[Index1]);
		const varying FVector3f X2 = VectorGather(&X[Index2]);
		const varying FVector3f X3 = VectorGather(&X[Index3]);
		
		const varying float Bary = Barys[Index];
		const varying float Dist = Dists[Index];
		const varying float Lambda = Lambdas[Index];
		
		const varying float Stiffness = StiffnessHasMap ? StiffnessOffsetRange.V[0] + StiffnessMapValues[Index]*StiffnessOffsetRange.V[1] : StiffnessOffsetRange.V[0];
		const varying float StiffnessWeft = StiffnessWeftHasMap ? StiffnessWeftOffsetRange.V[0] + StiffnessWeftMapValues[Index]*StiffnessWeftOffsetRange.V[1] : StiffnessWeftOffsetRange.V[0];
		const varying float StiffnessBias = StiffnessBiasHasMap ? StiffnessBiasOffsetRange.V[0] + StiffnessBiasMapValues[Index]*StiffnessBiasOffsetRange.V[1] : StiffnessBiasOffsetRange.V[0];

		const varying FVector3f StiffnessVector = {{StiffnessWeft, Stiffness, StiffnessBias}};
		const varying FVector3f WarpWeftBiasBaseMultiplier = VectorLoad(&WarpWeftBiasBaseMultiplers[extract(Index,0)]);
		const varying float StiffnessValue = VectorDot(WarpWeftBiasBaseMultiplier, StiffnessVector);

		const varying float DampingRatio = DampingHasMap ? DampingOffsetRange.V[0] + DampingMapValues[Index]*DampingOffsetRange.V[1] : DampingOffsetRange.V[0];

		varying float DLambda;
		const FVector3f Delta = GetXPBDAxialSpringDampingDelta(P1, P2, P3, X1, X2, X3, InvM1, InvM2, InvM3, Dt, Bary, Dist, Lambda, StiffnessValue, DampingRatio, DLambda);

		Lambdas[Index] = Lambda + DLambda;
		
		VectorScatter(&PandInvM[Index1], SetVector4( P1 + (InvM1 * Delta), InvM1 ));
		VectorScatter(&PandInvM[Index2], SetVector4( P2 - (InvM2 * Bary * Delta), InvM2 ));
		VectorScatter(&PandInvM[Index3], SetVector4( P3 - (InvM3 * (1.f - Bary) * Delta), InvM3 ));
	}
}
