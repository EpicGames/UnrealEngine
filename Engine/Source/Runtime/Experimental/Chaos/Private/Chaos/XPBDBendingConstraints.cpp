// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/XPBDBendingConstraints.h"
#include "Chaos/GraphColoring.h"
#include "Chaos/Framework/Parallel.h"
#include "ChaosStats.h"
#if INTEL_ISPC
#include "XPBDBendingConstraints.ispc.generated.h"
#endif

DECLARE_CYCLE_STAT(TEXT("Chaos XPBD Bending Constraint"), STAT_XPBD_Bending, STATGROUP_Chaos);

#if INTEL_ISPC && !UE_BUILD_SHIPPING
bool bChaos_XPBDBending_ISPC_Enabled = CHAOS_XPBD_BENDING_ISPC_ENABLED;
FAutoConsoleVariableRef CVarChaosXPBDBendingISPCEnabled(TEXT("p.Chaos.XPBDBending.ISPC"), bChaos_XPBDBending_ISPC_Enabled, TEXT("Whether to use ISPC optimizations in XPBD Bending constraints"));

static_assert(sizeof(ispc::FVector4f) == sizeof(Chaos::Softs::FPAndInvM), "sizeof(ispc::FVector4f) != sizeof(Chaos::Softs::FPAndInvM");
static_assert(sizeof(ispc::FVector3f) == sizeof(Chaos::Softs::FSolverVec3), "sizeof(ispc::FVector3f) != sizeof(Chaos::Softs::FSolverVec3");
static_assert(sizeof(ispc::FVector2f) == sizeof(Chaos::Softs::FSolverVec2), "sizeof(ispc::FVector2f) != sizeof(Chaos::Softs::FSolverVec2");
#endif

namespace Chaos::Softs {

// @todo(chaos): the parallel threshold (or decision to run parallel) should probably be owned by the solver and passed to the constraint container
int32 Chaos_XPBDBending_ParallelConstraintCount = 100;
FAutoConsoleVariableRef CVarChaosXPBDBendingParallelConstraintCount(TEXT("p.Chaos.XPBDBending.ParallelConstraintCount"), Chaos_XPBDBending_ParallelConstraintCount, TEXT("If we have more constraints than this, use parallel-for in Apply."));

#if !UE_BUILD_SHIPPING
bool bChaos_XPBDBending_SplitLambdaDamping = true;
FAutoConsoleVariableRef CVarChaosXPBDBendingSplitLambdaDamping(TEXT("p.Chaos.XPBDBending.SplitLambdaDamping"), bChaos_XPBDBending_SplitLambdaDamping, TEXT("Use the split two-pass damping model (slower but doesn't make cloth too soft at high damping levels)."));
#endif

template<typename SolverParticlesOrRange>
void FXPBDBendingConstraints::InitColor(const SolverParticlesOrRange& InParticles)
{
	// In dev builds we always color so we can tune the system without restarting. See Apply()
#if UE_BUILD_SHIPPING || UE_BUILD_TEST
	if (Constraints.Num() > Chaos_XPBDBending_ParallelConstraintCount)
#endif
	{
		const TArray<TArray<int32>> ConstraintsPerColor = FGraphColoring::ComputeGraphColoringParticlesOrRange(Constraints, InParticles, ParticleOffset, ParticleOffset + ParticleCount);

		// Reorder constraints based on color so each array in ConstraintsPerColor contains contiguous elements.
		TArray<TVec4<int32>> ReorderedConstraints; 
		TArray<TVec2<int32>> ReorderedConstraintSharedEdges;
		TArray<FSolverReal> ReorderedRestAngles;
		TArray<int32> OrigToReorderedIndices; // used to reorder stiffness indices
		ReorderedConstraints.SetNumUninitialized(Constraints.Num());
		ReorderedConstraintSharedEdges.SetNumUninitialized(Constraints.Num());
		ReorderedRestAngles.SetNumUninitialized(Constraints.Num());
		OrigToReorderedIndices.SetNumUninitialized(Constraints.Num());

		ConstraintsPerColorStartIndex.Reset(ConstraintsPerColor.Num() + 1);

		int32 ReorderedIndex = 0;
		for (const TArray<int32>& ConstraintsBatch : ConstraintsPerColor)
		{
			ConstraintsPerColorStartIndex.Add(ReorderedIndex);
			for (const int32& BatchConstraint : ConstraintsBatch)
			{
				const int32 OrigIndex = BatchConstraint;
				ReorderedConstraints[ReorderedIndex] = Constraints[OrigIndex];
				ReorderedConstraintSharedEdges[ReorderedIndex] = ConstraintSharedEdges[OrigIndex];
				ReorderedRestAngles[ReorderedIndex] = RestAngles[OrigIndex];
				OrigToReorderedIndices[OrigIndex] = ReorderedIndex;
				++ReorderedIndex;
			}
		}
		ConstraintsPerColorStartIndex.Add(ReorderedIndex);

		Constraints = MoveTemp(ReorderedConstraints);
		ConstraintSharedEdges = MoveTemp(ReorderedConstraintSharedEdges);
		RestAngles = MoveTemp(ReorderedRestAngles);
		XPBDStiffness.ReorderIndices(OrigToReorderedIndices);
		BucklingRatioWeighted.ReorderIndices(OrigToReorderedIndices);
		XPBDBucklingStiffness.ReorderIndices(OrigToReorderedIndices);

#if INTEL_ISPC
		ConstraintsIndex1.SetNumUninitialized(Constraints.Num());
		ConstraintsIndex2.SetNumUninitialized(Constraints.Num());
		ConstraintsIndex3.SetNumUninitialized(Constraints.Num());
		ConstraintsIndex4.SetNumUninitialized(Constraints.Num());
		for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
		{
			ConstraintsIndex1[ConstraintIndex] = Constraints[ConstraintIndex][0];
			ConstraintsIndex2[ConstraintIndex] = Constraints[ConstraintIndex][1];
			ConstraintsIndex3[ConstraintIndex] = Constraints[ConstraintIndex][2];
			ConstraintsIndex4[ConstraintIndex] = Constraints[ConstraintIndex][3];
		}
#endif
	}
}
template CHAOS_API void FXPBDBendingConstraints::InitColor(const FSolverParticles& InParticles);
template CHAOS_API void FXPBDBendingConstraints::InitColor(const FSolverParticlesRange& InParticles);


template<typename SolverParticlesOrRange>
void FXPBDBendingConstraints::Init(const SolverParticlesOrRange& InParticles)
{
	Lambdas.Reset();
	Lambdas.AddZeroed(Constraints.Num());
	LambdasDamping.Reset();
	LambdasDamping.AddZeroed(Constraints.Num());
#if INTEL_ISPC
	IsBuckled.SetNumUninitialized(Constraints.Num());
	X1Array.SetNumUninitialized(Constraints.Num());
	X2Array.SetNumUninitialized(Constraints.Num());
	X3Array.SetNumUninitialized(Constraints.Num());
	X4Array.SetNumUninitialized(Constraints.Num());
	if (bRealTypeCompatibleWithISPC && bChaos_Bending_ISPC_Enabled && ConstraintsIndex1.Num() == Constraints.Num())
	{
		if (BucklingRatioWeighted.HasWeightMap())
		{
			ispc::InitXPBDBendingConstraintsIsBuckledWithMaps(
				(const ispc::FVector3f*)InParticles.XArray().GetData(),
				ConstraintsIndex1.GetData(),
				ConstraintsIndex2.GetData(),
				ConstraintsIndex3.GetData(),
				ConstraintsIndex4.GetData(),
				RestAngles.GetData(),
				IsBuckled.GetData(),
				(ispc::FVector3f*)X1Array.GetData(),
				(ispc::FVector3f*)X2Array.GetData(),
				(ispc::FVector3f*)X3Array.GetData(),
				(ispc::FVector3f*)X4Array.GetData(),
				BucklingRatioWeighted.GetIndices().GetData(),
				BucklingRatioWeighted.GetTable().GetData(),
				Constraints.Num()
			);
		}
		else
		{
			ispc::InitXPBDBendingConstraintsIsBuckled(
				(const ispc::FVector3f*)InParticles.XArray().GetData(),
				ConstraintsIndex1.GetData(),
				ConstraintsIndex2.GetData(),
				ConstraintsIndex3.GetData(),
				ConstraintsIndex4.GetData(),
				RestAngles.GetData(),
				IsBuckled.GetData(),
				(ispc::FVector3f*)X1Array.GetData(),
				(ispc::FVector3f*)X2Array.GetData(),
				(ispc::FVector3f*)X3Array.GetData(),
				(ispc::FVector3f*)X4Array.GetData(),
				(FSolverReal)BucklingRatioWeighted,
				Constraints.Num()
			);
		}
	}
	else
#endif
	{
		FPBDBendingConstraintsBase::Init(InParticles);
	}
}
template void CHAOS_API FXPBDBendingConstraints::Init(const FSolverParticles& InParticles);
template void CHAOS_API FXPBDBendingConstraints::Init(const FSolverParticlesRange& InParticles);

void FXPBDBendingConstraints::SetProperties(
	const FCollectionPropertyConstFacade& PropertyCollection,
	const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps)
{
	if (IsXPBDBendingElementStiffnessMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedValue(GetWeightedFloatXPBDBendingElementStiffness(PropertyCollection).ClampAxes(0, MaxStiffness));
		if (IsXPBDBendingElementStiffnessStringDirty(PropertyCollection))
		{
			const FString& WeightMapName = GetXPBDBendingElementStiffnessString(PropertyCollection);
			XPBDStiffness = FPBDFlatWeightMap(
				WeightedValue,
				WeightMaps.FindRef(WeightMapName),
				TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
				ParticleOffset,
				ParticleCount);
		}
		else
		{
			XPBDStiffness.SetWeightedValue(WeightedValue);
		}
	}
	if (IsXPBDBucklingRatioMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedValue = FSolverVec2(GetWeightedFloatXPBDBucklingRatio(PropertyCollection)).ClampAxes(0.f, 1.f);
		if (IsXPBDBucklingRatioStringDirty(PropertyCollection))
		{
			const FString& WeightMapName = GetXPBDBucklingRatioString(PropertyCollection);
			BucklingRatioWeighted = FPBDWeightMap(
				WeightedValue,
				WeightMaps.FindRef(WeightMapName),
				TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
				ParticleOffset,
				ParticleCount);
		}
		else
		{
			BucklingRatioWeighted.SetWeightedValue(WeightedValue);
		}
	}
	if (IsXPBDBucklingStiffnessMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedValue(GetWeightedFloatXPBDBucklingStiffness(PropertyCollection).ClampAxes(0, MaxStiffness));
		if (IsXPBDBucklingStiffnessStringDirty(PropertyCollection))
		{
			const FString& WeightMapName = GetXPBDBucklingStiffnessString(PropertyCollection);
			XPBDBucklingStiffness = FPBDFlatWeightMap(
				WeightedValue,
				WeightMaps.FindRef(WeightMapName),
				TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
				ParticleOffset,
				ParticleCount);
		}
		else
		{
			XPBDBucklingStiffness.SetWeightedValue(WeightedValue);
		}
	}
	if (IsXPBDBendingElementDampingMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedValue = FSolverVec2(GetWeightedFloatXPBDBendingElementDamping(PropertyCollection)).ClampAxes(MinDamping, MaxDamping);
		if (IsXPBDBendingElementDampingStringDirty(PropertyCollection))
		{
			const FString& WeightMapName = GetXPBDBendingElementDampingString(PropertyCollection);
			DampingRatio = FPBDFlatWeightMap(
				WeightedValue,
				WeightMaps.FindRef(WeightMapName),
				TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
				ParticleOffset,
				ParticleCount);
		}
		else
		{
			DampingRatio.SetWeightedValue(WeightedValue);
		}
	}
}

template<bool bDampingOnly, bool bElasticOnly, typename SolverParticlesOrRange>
void FXPBDBendingConstraints::ApplyHelper(SolverParticlesOrRange& Particles, const FSolverReal Dt, const int32 ConstraintIndex, const FSolverReal StiffnessValue, const FSolverReal BucklingValue, const FSolverReal DampingRatioValue) const
{
	check(!(bDampingOnly && bElasticOnly));

	const TVec4<int32>& Constraint = Constraints[ConstraintIndex];
	const int32 Index1 = Constraint[0];
	const int32 Index2 = Constraint[1];
	const int32 Index3 = Constraint[2];
	const int32 Index4 = Constraint[3];

	const FSolverReal BiphasicStiffnessValue = IsBuckled[ConstraintIndex] ? BucklingValue : StiffnessValue;
	const FSolverReal CombinedInvMass = Particles.InvM(Index1) + Particles.InvM(Index2) + Particles.InvM(Index3) + Particles.InvM(Index4);
	check(CombinedInvMass > 0);
	const FSolverReal Damping = (FSolverReal)2.f * DampingRatioValue * FMath::Sqrt(BiphasicStiffnessValue / CombinedInvMass);

	const FSolverReal Angle = CalcAngle(Particles.P(Index1), Particles.P(Index2), Particles.P(Index3), Particles.P(Index4));
	const TStaticArray<FSolverVec3, 4> Grads = Base::GetGradients(Particles, ConstraintIndex);

	const FSolverVec3 V1TimesDt = Particles.P(Index1) - Particles.GetX(Index1);
	const FSolverVec3 V2TimesDt = Particles.P(Index2) - Particles.GetX(Index2);
	const FSolverVec3 V3TimesDt = Particles.P(Index3) - Particles.GetX(Index3);
	const FSolverVec3 V4TimesDt = Particles.P(Index4) - Particles.GetX(Index4);

	FSolverReal& Lambda = bDampingOnly ? LambdasDamping[ConstraintIndex] : Lambdas[ConstraintIndex];
	const FSolverReal AlphaInv = bDampingOnly ? (FSolverReal)0.f : BiphasicStiffnessValue * Dt * Dt;
	const FSolverReal BetaDt = bElasticOnly ? (FSolverReal)0.f : Damping * Dt;

	const FSolverReal DampingTerm = bElasticOnly ? (FSolverReal)0.f :  BetaDt * (FSolverVec3::DotProduct(V1TimesDt, Grads[0]) + FSolverVec3::DotProduct(V2TimesDt, Grads[1]) + FSolverVec3::DotProduct(V3TimesDt, Grads[2]) + FSolverVec3::DotProduct(V4TimesDt, Grads[3]));
	const FSolverReal ElasticTerm = bDampingOnly ? (FSolverReal)0.f : AlphaInv * (Angle - RestAngles[ConstraintIndex]);

	const FSolverReal Denom = (AlphaInv + BetaDt) * (Particles.InvM(Index1) * Grads[0].SizeSquared() + Particles.InvM(Index2) * Grads[1].SizeSquared() + Particles.InvM(Index3) * Grads[2].SizeSquared() + Particles.InvM(Index4) * Grads[3].SizeSquared()) + (FSolverReal)1.;
	const FSolverReal DLambda = (-ElasticTerm - DampingTerm - Lambda) / Denom;

	Particles.P(Index1) += DLambda * Particles.InvM(Index1) * Grads[0];
	Particles.P(Index2) += DLambda * Particles.InvM(Index2) * Grads[1];
	Particles.P(Index3) += DLambda * Particles.InvM(Index3) * Grads[2];
	Particles.P(Index4) += DLambda * Particles.InvM(Index4) * Grads[3];
	Lambda += DLambda;
}

template<typename SolverParticlesOrRange>
void FXPBDBendingConstraints::Apply(SolverParticlesOrRange& Particles, const FSolverReal Dt) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FXPBDBendingConstraints_Apply);
	SCOPE_CYCLE_COUNTER(STAT_XPBD_Bending);

	const bool StiffnessHasWeightMap = XPBDStiffness.HasWeightMap();
	const bool BucklingStiffnessHasWeightMap = XPBDBucklingStiffness.HasWeightMap();
	const bool DampingHasWeightMap = DampingRatio.HasWeightMap();

	if (ConstraintsPerColorStartIndex.Num() > 0 && Constraints.Num() > Chaos_XPBDBending_ParallelConstraintCount)
	{
		const int32 ConstraintColorNum = ConstraintsPerColorStartIndex.Num() - 1;

#if INTEL_ISPC
		if (bRealTypeCompatibleWithISPC && bChaos_XPBDBending_ISPC_Enabled)
		{
			if (!StiffnessHasWeightMap && !BucklingStiffnessHasWeightMap && !DampingHasWeightMap)
			{
				const FSolverReal ExpStiffnessValue = (FSolverReal)XPBDStiffness;
				const FSolverReal ExpBucklingValue = (FSolverReal)XPBDBucklingStiffness;
				const FSolverReal DampingRatioValue = (FSolverReal)DampingRatio;

				if (ExpStiffnessValue <= MinStiffness && ExpBucklingValue <= MinStiffness)
				{
					return;
				}
				if (DampingRatioValue > 0)
				{
					if (bChaos_XPBDBending_SplitLambdaDamping)
					{
						for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
						{
							const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
							const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
							ispc::ApplyXPBDBendingDampingConstraints(
								(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
								(const ispc::FVector3f*)&X1Array.GetData()[ColorStart],
								(const ispc::FVector3f*)&X2Array.GetData()[ColorStart],
								(const ispc::FVector3f*)&X3Array.GetData()[ColorStart],
								(const ispc::FVector3f*)&X4Array.GetData()[ColorStart],
								&ConstraintsIndex1.GetData()[ColorStart],
								&ConstraintsIndex2.GetData()[ColorStart],
								&ConstraintsIndex3.GetData()[ColorStart],
								&ConstraintsIndex4.GetData()[ColorStart],
								&RestAngles.GetData()[ColorStart],
								&IsBuckled.GetData()[ColorStart],
								&LambdasDamping.GetData()[ColorStart],
								Dt,
								ExpStiffnessValue,
								ExpBucklingValue,
								DampingRatioValue,
								ColorSize);
						}
					}
					else
					{
						for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
						{
							const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
							const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
							ispc::ApplyXPBDBendingConstraintsWithDamping(
								(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
								(const ispc::FVector3f*)&X1Array.GetData()[ColorStart],
								(const ispc::FVector3f*)&X2Array.GetData()[ColorStart],
								(const ispc::FVector3f*)&X3Array.GetData()[ColorStart],
								(const ispc::FVector3f*)&X4Array.GetData()[ColorStart],
								&ConstraintsIndex1.GetData()[ColorStart],
								&ConstraintsIndex2.GetData()[ColorStart],
								&ConstraintsIndex3.GetData()[ColorStart],
								&ConstraintsIndex4.GetData()[ColorStart],
								&RestAngles.GetData()[ColorStart],
								&IsBuckled.GetData()[ColorStart],
								&Lambdas.GetData()[ColorStart],
								Dt,
								ExpStiffnessValue,
								ExpBucklingValue,
								DampingRatioValue,
								ColorSize);
						}
						return;
					}
				}
				for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
				{
					const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
					const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
					ispc::ApplyXPBDBendingConstraints(
						(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
						&ConstraintsIndex1.GetData()[ColorStart],
						&ConstraintsIndex2.GetData()[ColorStart],
						&ConstraintsIndex3.GetData()[ColorStart],
						&ConstraintsIndex4.GetData()[ColorStart],
						&RestAngles.GetData()[ColorStart],
						&IsBuckled.GetData()[ColorStart],
						&Lambdas.GetData()[ColorStart],
						Dt,
						ExpStiffnessValue,
						ExpBucklingValue,
						ColorSize);
				}
			}
			else
			{
				// ISPC with maps
				if (DampingHasWeightMap || (FSolverReal)DampingRatio > 0)
				{
					if (bChaos_XPBDBending_SplitLambdaDamping)
					{
						for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
						{
							const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
							const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
							ispc::ApplyXPBDBendingDampingConstraintsWithMaps(
								(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
								(const ispc::FVector3f*)&X1Array.GetData()[ColorStart],
								(const ispc::FVector3f*)&X2Array.GetData()[ColorStart],
								(const ispc::FVector3f*)&X3Array.GetData()[ColorStart],
								(const ispc::FVector3f*)&X4Array.GetData()[ColorStart],
								&ConstraintsIndex1.GetData()[ColorStart],
								&ConstraintsIndex2.GetData()[ColorStart],
								&ConstraintsIndex3.GetData()[ColorStart],
								&ConstraintsIndex4.GetData()[ColorStart],
								&RestAngles.GetData()[ColorStart],
								&IsBuckled.GetData()[ColorStart],
								&LambdasDamping.GetData()[ColorStart],
								Dt,
								StiffnessHasWeightMap,
								reinterpret_cast<const ispc::FVector2f&>(XPBDStiffness.GetOffsetRange()),
								StiffnessHasWeightMap ? &XPBDStiffness.GetMapValues().GetData()[ColorStart] : nullptr,
								BucklingStiffnessHasWeightMap,
								reinterpret_cast<const ispc::FVector2f&>(XPBDBucklingStiffness.GetOffsetRange()),
								BucklingStiffnessHasWeightMap ? &XPBDBucklingStiffness.GetMapValues().GetData()[ColorStart] : nullptr,
								DampingHasWeightMap,
								reinterpret_cast<const ispc::FVector2f&>(DampingRatio.GetOffsetRange()),
								DampingHasWeightMap ? &DampingRatio.GetMapValues()[ColorStart] : nullptr,
								ColorSize);
						}
					}
					else
					{
						for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
						{
							const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
							const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
							ispc::ApplyXPBDBendingConstraintsWithDampingAndMaps(
								(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
								(const ispc::FVector3f*)&X1Array.GetData()[ColorStart],
								(const ispc::FVector3f*)&X2Array.GetData()[ColorStart],
								(const ispc::FVector3f*)&X3Array.GetData()[ColorStart],
								(const ispc::FVector3f*)&X4Array.GetData()[ColorStart],
								&ConstraintsIndex1.GetData()[ColorStart],
								&ConstraintsIndex2.GetData()[ColorStart],
								&ConstraintsIndex3.GetData()[ColorStart],
								&ConstraintsIndex4.GetData()[ColorStart],
								&RestAngles.GetData()[ColorStart],
								&IsBuckled.GetData()[ColorStart],
								&Lambdas.GetData()[ColorStart],
								Dt,
								StiffnessHasWeightMap,
								reinterpret_cast<const ispc::FVector2f&>(XPBDStiffness.GetOffsetRange()),
								StiffnessHasWeightMap ? &XPBDStiffness.GetMapValues().GetData()[ColorStart] : nullptr,
								BucklingStiffnessHasWeightMap,
								reinterpret_cast<const ispc::FVector2f&>(XPBDBucklingStiffness.GetOffsetRange()),
								BucklingStiffnessHasWeightMap ? &XPBDBucklingStiffness.GetMapValues().GetData()[ColorStart] : nullptr,
								DampingHasWeightMap,
								reinterpret_cast<const ispc::FVector2f&>(DampingRatio.GetOffsetRange()),
								DampingHasWeightMap ? &DampingRatio.GetMapValues()[ColorStart] : nullptr,
								ColorSize);
						}
						return;
					}
				}
				for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
				{
					const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
					const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
					ispc::ApplyXPBDBendingConstraintsWithMaps(
						(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
						&ConstraintsIndex1.GetData()[ColorStart],
						&ConstraintsIndex2.GetData()[ColorStart],
						&ConstraintsIndex3.GetData()[ColorStart],
						&ConstraintsIndex4.GetData()[ColorStart],
						&RestAngles.GetData()[ColorStart],
						&IsBuckled.GetData()[ColorStart],
						&Lambdas.GetData()[ColorStart],
						Dt,
						StiffnessHasWeightMap,
						reinterpret_cast<const ispc::FVector2f&>(XPBDStiffness.GetOffsetRange()),
						StiffnessHasWeightMap ? &XPBDStiffness.GetMapValues().GetData()[ColorStart] : nullptr,
						BucklingStiffnessHasWeightMap,
						reinterpret_cast<const ispc::FVector2f&>(XPBDBucklingStiffness.GetOffsetRange()),
						BucklingStiffnessHasWeightMap ? &XPBDBucklingStiffness.GetMapValues().GetData()[ColorStart] : nullptr,
						ColorSize);
				}
			}
		}
		else
#endif
		{
			// Parallel non-ispc
			const FSolverReal StiffnessNoMap = (FSolverReal)XPBDStiffness;
			const FSolverReal BucklingStiffnessNoMap = (FSolverReal)XPBDBucklingStiffness;
			const FSolverReal DampingNoMap = (FSolverReal)DampingRatio;

			if (DampingHasWeightMap || (FSolverReal)DampingRatio > 0)
			{
				if (bChaos_XPBDBending_SplitLambdaDamping)
				{
					for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
					{
						const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
						const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
						PhysicsParallelFor(ColorSize, [&](const int32 Index)
						{
							const int32 ConstraintIndex = ColorStart + Index;
							const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? XPBDStiffness[ConstraintIndex] : StiffnessNoMap;
							const FSolverReal ExpBucklingValue = BucklingStiffnessHasWeightMap ? XPBDBucklingStiffness[ConstraintIndex] : BucklingStiffnessNoMap;
							const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
							constexpr bool bDampingOnly = true;
							constexpr bool bElasticOnly = false;
							ApplyHelper<bDampingOnly, bElasticOnly>(Particles, Dt, ConstraintIndex, ExpStiffnessValue, ExpBucklingValue, DampingRatioValue);
						});
					}
				}
				else
				{
					for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
					{
						const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
						const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
						PhysicsParallelFor(ColorSize, [&](const int32 Index)
						{
							const int32 ConstraintIndex = ColorStart + Index;
							const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? XPBDStiffness[ConstraintIndex] : StiffnessNoMap;
							const FSolverReal ExpBucklingValue = BucklingStiffnessHasWeightMap ? XPBDBucklingStiffness[ConstraintIndex] : BucklingStiffnessNoMap;
							const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
							constexpr bool bDampingOnly = false;
							constexpr bool bElasticOnly = false;
							ApplyHelper<bDampingOnly, bElasticOnly>(Particles, Dt, ConstraintIndex, ExpStiffnessValue, ExpBucklingValue, DampingRatioValue);
						});
					}
					return;
				}
			}
			for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
			{
				const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
				const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
				PhysicsParallelFor(ColorSize, [&](const int32 Index)
				{
					const int32 ConstraintIndex = ColorStart + Index;
					const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? XPBDStiffness[ConstraintIndex] : StiffnessNoMap;
					const FSolverReal ExpBucklingValue = BucklingStiffnessHasWeightMap ? XPBDBucklingStiffness[ConstraintIndex] : BucklingStiffnessNoMap;
					const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
					constexpr bool bDampingOnly = false;
					constexpr bool bElasticOnly = true;
					ApplyHelper<bDampingOnly, bElasticOnly>(Particles, Dt, ConstraintIndex, ExpStiffnessValue, ExpBucklingValue, DampingRatioValue);
				});
			}
		}
	}
	else
	{
		// Single-threaded
		const FSolverReal StiffnessNoMap = (FSolverReal)XPBDStiffness;
		const FSolverReal BucklingStiffnessNoMap = (FSolverReal)XPBDBucklingStiffness;
		const FSolverReal DampingNoMap = (FSolverReal)DampingRatio;

		if (DampingHasWeightMap || (FSolverReal)DampingRatio > 0)
		{
			if (bChaos_XPBDBending_SplitLambdaDamping)
			{
				for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
				{
					const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? XPBDStiffness[ConstraintIndex] : StiffnessNoMap;
					const FSolverReal ExpBucklingValue = BucklingStiffnessHasWeightMap ? XPBDBucklingStiffness[ConstraintIndex] : BucklingStiffnessNoMap;
					const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
					constexpr bool bDampingOnly = true;
					constexpr bool bElasticOnly = false;
					ApplyHelper<bDampingOnly, bElasticOnly>(Particles, Dt, ConstraintIndex, ExpStiffnessValue, ExpBucklingValue, DampingRatioValue);
				}
			}
			else
			{
				for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
				{
					const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? XPBDStiffness[ConstraintIndex] : StiffnessNoMap;
					const FSolverReal ExpBucklingValue = BucklingStiffnessHasWeightMap ? XPBDBucklingStiffness[ConstraintIndex] : BucklingStiffnessNoMap;
					const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
					constexpr bool bDampingOnly = false;
					constexpr bool bElasticOnly = false;
					ApplyHelper<bDampingOnly, bElasticOnly>(Particles, Dt, ConstraintIndex, ExpStiffnessValue, ExpBucklingValue, DampingRatioValue);
				}
				return;
			}
		}
		for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
		{
			const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? XPBDStiffness[ConstraintIndex] : StiffnessNoMap;
			const FSolverReal ExpBucklingValue = BucklingStiffnessHasWeightMap ? XPBDBucklingStiffness[ConstraintIndex] : BucklingStiffnessNoMap;
			const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
			constexpr bool bDampingOnly = false;
			constexpr bool bElasticOnly = true;
			ApplyHelper<bDampingOnly, bElasticOnly>(Particles, Dt, ConstraintIndex, ExpStiffnessValue, ExpBucklingValue, DampingRatioValue);
		}
	}
}
template CHAOS_API void FXPBDBendingConstraints::Apply(FSolverParticles& Particles, const FSolverReal Dt) const;
template CHAOS_API void FXPBDBendingConstraints::Apply(FSolverParticlesRange& Particles, const FSolverReal Dt) const;

FSolverReal FXPBDBendingConstraints::ComputeTotalEnergy(const FSolverParticles& InParticles, const FSolverReal ExplicitStiffness)
{
	FSolverReal StiffnessValue = (FSolverReal)XPBDStiffness;
	if (ExplicitStiffness > 0.f)
	{
		StiffnessValue = ExplicitStiffness;
	}
	const FSolverReal DampingRatioValue = (FSolverReal)DampingRatio;

	auto SafeRecip = [](const FSolverReal Len, const FSolverReal Fallback)
	{
		if (Len > UE_SMALL_NUMBER)
		{
			return 1.f / Len;
		}
		return Fallback;
	};

	FSolverReal Energy = 0.f;

	for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ConstraintIndex++)
	{
		const TVec4<int32>& Constraint = Constraints[ConstraintIndex];
		const FSolverVec3& P1 = InParticles.P(Constraint[0]);
		const FSolverVec3& P2 = InParticles.P(Constraint[1]);
		const FSolverVec3& P3 = InParticles.P(Constraint[2]);
		const FSolverVec3& P4 = InParticles.P(Constraint[3]);

		const FSolverVec3 Y1 = P2 - P1;
		const FSolverVec3 Y2 = P3 - P1;
		const FSolverVec3 Y3 = P4 - P1;
		const FSolverVec3 T1 = Y1.GetSafeNormal();

		const FSolverVec3 Z0 = Y2 - FSolverVec3::DotProduct(Y2, T1) * T1;
		const FSolverVec3 Z1 = Y3 - FSolverVec3::DotProduct(Y3, T1) * T1;

		const FSolverVec3 Z0Normalized = Z0.GetSafeNormal();
		const FSolverVec3 Z1Normalized = Z1.GetSafeNormal();

		const FSolverVec3 Tb = FSolverVec3::CrossProduct(Z0, Z1).GetSafeNormal();

		const FSolverReal S = FSolverVec3::DotProduct(Tb, T1);

		FSolverReal Theta = FMath::Acos(FSolverVec3::DotProduct(Z0Normalized, Z1Normalized));

		Theta = S * (PI - Theta);

		const FSolverReal CAngle = Theta - RestAngles[ConstraintIndex];


		Energy += CAngle * CAngle * StiffnessValue / 2.f;

	}

	return Energy;

}


void ComputeGradTheta(const FSolverVec3& X0, const FSolverVec3& X1, const FSolverVec3& X2, const FSolverVec3& X3, const int32 Index, FSolverVec3& dThetadx, FSolverReal& Theta) 
{
	const FSolverVec3 E21 = X2 - X1;
	const FSolverReal Norme = E21.Length();
	const FSolverVec3 E10 = X1 - X0;
	const FSolverVec3 E20 = X2 - X0;
	const FSolverVec3 N0 = FSolverVec3::CrossProduct(E20, E10);
	const FSolverReal SquaredNorm0 = N0.SquaredLength();

	const FSolverVec3 E23 = X2 - X3;
	const FSolverVec3 E13 = X1 - X3;
	const FSolverVec3 N1 = FSolverVec3::CrossProduct(E13, E23); 
	const FSolverReal SquaredNorm1 = N1.SquaredLength();

	const FSolverVec3 N0CrossN1 = FSolverVec3::CrossProduct(N0, N1);
	Theta = FMath::Atan2(FSolverVec3::DotProduct(N0CrossN1, E21) / Norme, N0.Dot(N1)); 

	switch (Index)
		{
			case 0:
				dThetadx = -(Norme / SquaredNorm0) * N0;
				break;
			case 1:
				dThetadx = ((E20.Dot(E21)) / (Norme * SquaredNorm1)) * N1 + ((E23.Dot(E21)) / (Norme * SquaredNorm0)) * N0;
				break;
			case 2:
				dThetadx = -((E10.Dot(E21)) / (Norme * SquaredNorm1)) * N1 - ((E13.Dot(E21)) / (Norme * SquaredNorm0)) * N0;
				break;
			case 3:
				dThetadx = -(Norme / SquaredNorm1) * N1;
				break;
			default:
				break;
		}
}


void FXPBDBendingConstraints::AddBendingResidualAndHessian(const FSolverParticlesRange& Particles, const int32 ConstraintIndex, const int32 ConstraintIndexLocal, const FSolverReal Dt, TVec3<FSolverReal>& ParticleResidual, Chaos::PMatrix<FSolverReal, 3, 3>& ParticleHessian)
{
	const FSolverReal ExpStiffnessValue = (FSolverReal)XPBDStiffness;
	const FSolverReal ExpBucklingValue = (FSolverReal)XPBDBucklingStiffness;

	const TVec4<int32>& Constraint = Constraints[ConstraintIndex];
	const FSolverVec3& P1 = Particles.P(Constraint[0]);
	const FSolverVec3& P2 = Particles.P(Constraint[1]);
	const FSolverVec3& P3 = Particles.P(Constraint[2]);
	const FSolverVec3& P4 = Particles.P(Constraint[3]);

	FSolverVec3 DThetaDx(0.f);
	FSolverReal Theta = 0.f;

	constexpr int32 LocalIndexMap[] = { 2, 1, 0, 3 };
	const int32 ActualConstraintIndexLocal = LocalIndexMap[ConstraintIndexLocal];

	ComputeGradTheta(P3, P2, P1, P4, ActualConstraintIndexLocal, DThetaDx, Theta);

	const FSolverReal CAngle = Theta - RestAngles[ConstraintIndex];

	const FSolverReal BiphasicStiffnessValue = IsBuckled[ConstraintIndex] ? ExpBucklingValue : ExpStiffnessValue;

	ParticleResidual -= Dt * Dt * BiphasicStiffnessValue * CAngle * DThetaDx;

	for (int32 Alpha = 0; Alpha < 3; Alpha++)
	{
		ParticleHessian.SetRow(Alpha, ParticleHessian.GetRow(Alpha) + Dt * Dt * BiphasicStiffnessValue * DThetaDx[Alpha] * DThetaDx);
	}
}

void FXPBDBendingConstraints::AddBendingResidualAndHessian(const FSolverParticles& Particles, const int32 ConstraintIndex, const int32 ConstraintIndexLocal, const FSolverReal Dt, TVec3<FSolverReal>& ParticleResidual, Chaos::PMatrix<FSolverReal, 3, 3>& ParticleHessian)
{
	AddBendingResidualAndHessian(FSolverParticlesRange(Particles), ConstraintIndex, ConstraintIndexLocal, Dt, ParticleResidual, ParticleHessian);
}


void FXPBDBendingConstraints::AddInternalForceDifferential(const FSolverParticlesRange& InParticles, const TArray<TVector<FSolverReal, 3>>& DeltaParticles, TArray<TVector<FSolverReal, 3>>& Ndf)
{
	const FSolverReal ExpStiffnessValue = (FSolverReal)XPBDStiffness;
	const FSolverReal ExpBucklingValue = (FSolverReal)XPBDBucklingStiffness;

	ensure(Ndf.Num() == InParticles.Size());

	for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ConstraintIndex++)
	{
		const TVec4<int32>& Constraint = Constraints[ConstraintIndex];
		const FSolverVec3& P1 = InParticles.P(Constraint[0]);
		const FSolverVec3& P2 = InParticles.P(Constraint[1]);
		const FSolverVec3& P3 = InParticles.P(Constraint[2]);
		const FSolverVec3& P4 = InParticles.P(Constraint[3]);

		FSolverReal Theta = 0.f;

		TArray<FSolverVec3> Alldthetadx;
		Alldthetadx.Init(FSolverVec3(0.f), 4);

		const TArray<int32> LocalIndexMap = { 2, 1, 0, 3 };
		for (int32 Index = 0; Index < 4; Index++)
		{
			const int32 ActualConstraintIndexLocal = LocalIndexMap[Index];
			ComputeGradTheta(P3, P2, P1, P4, ActualConstraintIndexLocal, Alldthetadx[Index], Theta);
		}

		const FSolverReal BiphasicStiffnessValue = IsBuckled[ConstraintIndex] ? ExpBucklingValue : ExpStiffnessValue;

		FSolverReal DeltaC = 0.f;

		for (int32 Index = 0; Index < 4; Index++)
		{
			for (int32 Alpha = 0; Alpha < 3; Alpha++)
			{
				DeltaC += Alldthetadx[Index][Alpha] * DeltaParticles[Constraint[Index]][Alpha];
			}
		}

		for (int32 Index = 0; Index < 4; Index++)
		{
			Ndf[Constraint[Index]] += BiphasicStiffnessValue * DeltaC * Alldthetadx[Index];
		}

	}
}

void FXPBDBendingConstraints::AddInternalForceDifferential(const FSolverParticles& InParticles, const TArray<TVector<FSolverReal, 3>>& DeltaParticles, TArray<TVector<FSolverReal, 3>>& Ndf)
{
	AddInternalForceDifferential(FSolverParticlesRange(InParticles), DeltaParticles, Ndf);
}

}  // End namespace Chaos::Softs
