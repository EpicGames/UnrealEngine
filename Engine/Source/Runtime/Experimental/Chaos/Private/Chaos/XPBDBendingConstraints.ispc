// Copyright Epic Games, Inc. All Rights Reserved.

#define EXPLICIT_VECTOR4 1

#include "Math/Vector.isph"
#include "Chaos/PBDSofts.isph"
#include "Chaos/XPBDBendingElement.isph"

static inline float CalcAngle(const FVector3f& P1, const FVector3f& P2, const FVector3f& P3, const FVector3f& P4)
{
	const FVector3f SharedEdgeNormalized = VectorGetSafeNormal(P2 - P1);
	const FVector3f P13CrossP23 = VectorCross(P1 - P3, P2 - P3);
	const float Normal1Len = VectorSize(P13CrossP23);
	const FVector3f Normal1 = SafeDivide(P13CrossP23, Normal1Len);
	const FVector3f P24CrossP14 = VectorCross(P2 - P4, P1 - P4);
	const float Normal2Len = VectorSize(P24CrossP14);
	const FVector3f Normal2 = SafeDivide(P24CrossP14, Normal2Len);

	const FVector3f N2CrossN1 = VectorCross(Normal2, Normal1);

	const float CosPhi = clamp(VectorDot(Normal1, Normal2), (uniform float)(-1), (uniform float)(1));
	const float SinPhi = clamp(VectorDot(N2CrossN1, SharedEdgeNormalized), (uniform float)(-1), (uniform float)(1));
	return atan2(SinPhi, CosPhi);
}

export void InitXPBDBendingConstraintsIsBuckled(const uniform FVector3f X[],
							const uniform int32 ConstraintIndex1[],
							const uniform int32 ConstraintIndex2[],
							const uniform int32 ConstraintIndex3[],
							const uniform int32 ConstraintIndex4[],
							const uniform float RestAngles[],
							uniform bool IsBuckled[],
							uniform FVector3f X1Out[],
							uniform FVector3f X2Out[],
							uniform FVector3f X3Out[],
							uniform FVector3f X4Out[],
							const uniform float BucklingRatio,
							const uniform int32 NumConstraints)
{
	foreach(i = 0 ... NumConstraints)
	{
		const varying int32 Index1 = ConstraintIndex1[i];
		const varying int32 Index2 = ConstraintIndex2[i];
		const varying int32 Index3 = ConstraintIndex3[i];
		const varying int32 Index4 = ConstraintIndex4[i];

		const varying FVector3f X1 = VectorGather(&X[Index1]);
		const varying FVector3f X2 = VectorGather(&X[Index2]);
		const varying FVector3f X3 = VectorGather(&X[Index3]);
		const varying FVector3f X4 = VectorGather(&X[Index4]);

		const varying float RestAngle = RestAngles[i];
		const varying float BucklingAngle = BucklingRatio * (FLOAT_PI - abs(RestAngle));

		const varying float Angle = CalcAngle(X1, X2, X3, X4);
		IsBuckled[i] = FLOAT_PI - abs(Angle) < BucklingAngle;

		VectorStore(&X1Out[extract(i,0)], X1);
		VectorStore(&X2Out[extract(i,0)], X2);
		VectorStore(&X3Out[extract(i,0)], X3);
		VectorStore(&X4Out[extract(i,0)], X4);
	}
}

export void InitXPBDBendingConstraintsIsBuckledWithMaps(const uniform FVector3f X[],
							const uniform int32 ConstraintIndex1[],
							const uniform int32 ConstraintIndex2[],
							const uniform int32 ConstraintIndex3[],
							const uniform int32 ConstraintIndex4[],
							const uniform float RestAngles[],
							uniform bool IsBuckled[],
							uniform FVector3f X1Out[],
							uniform FVector3f X2Out[],
							uniform FVector3f X3Out[],
							uniform FVector3f X4Out[],
							const uniform uint8 BucklingRatioIndices[],
							const uniform float BucklingRatioTable[],
							const uniform int32 NumConstraints)
{
	foreach(i = 0 ... NumConstraints)
	{
		const varying int32 Index1 = ConstraintIndex1[i];
		const varying int32 Index2 = ConstraintIndex2[i];
		const varying int32 Index3 = ConstraintIndex3[i];
		const varying int32 Index4 = ConstraintIndex4[i];
		const varying int8 BucklingRatioIndex = BucklingRatioIndices[i];

		const varying FVector3f X1 = VectorGather(&X[Index1]);
		const varying FVector3f X2 = VectorGather(&X[Index2]);
		const varying FVector3f X3 = VectorGather(&X[Index3]);
		const varying FVector3f X4 = VectorGather(&X[Index4]);
		
		#pragma ignore warning(perf)
		const varying float BucklingRatio = BucklingRatioTable[BucklingRatioIndex];
		const varying float RestAngle = RestAngles[i];
		const varying float BucklingAngle = BucklingRatio * (FLOAT_PI - abs(RestAngle));

		const varying float Angle = CalcAngle(X1, X2, X3, X4);
		IsBuckled[i] = FLOAT_PI - abs(Angle) < BucklingAngle;

		VectorStore(&X1Out[extract(i,0)], X1);
		VectorStore(&X2Out[extract(i,0)], X2);
		VectorStore(&X3Out[extract(i,0)], X3);
		VectorStore(&X4Out[extract(i,0)], X4);
	}
}

export void ApplyXPBDBendingConstraints(uniform FVector4f PandInvM[],
									const uniform int32 ConstraintIndex1[],
									const uniform int32 ConstraintIndex2[],
									const uniform int32 ConstraintIndex3[],
									const uniform int32 ConstraintIndex4[],
									const uniform float RestAngles[],
									const uniform bool IsBuckled[],
									uniform float Lambdas[],
									const uniform float Dt,
									const uniform float Stiffness,
									const uniform float BucklingStiffness,
									const uniform int32 NumConstraints)
{

	foreach(i = 0 ... NumConstraints)
	{
		const varying int32 i1 = ConstraintIndex1[i];
		const varying int32 i2 = ConstraintIndex2[i];
		const varying int32 i3 = ConstraintIndex3[i];
		const varying int32 i4 = ConstraintIndex4[i];
		
		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[i1]);
		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[i2]);
		const varying FVector4f PandInvM3 = VectorGather(&PandInvM[i3]);
		const varying FVector4f PandInvM4 = VectorGather(&PandInvM[i4]);

		varying FVector3f P1, P2, P3, P4;
		varying float M1, M2, M3, M4;

		UnzipPandInvM(PandInvM1, P1, M1);
		UnzipPandInvM(PandInvM2, P2, M2);
		UnzipPandInvM(PandInvM3, P3, M3);
		UnzipPandInvM(PandInvM4, P4, M4);

		const varying float StiffnessValue = select(IsBuckled[i], BucklingStiffness, Stiffness);
		
		const varying float Lambda = Lambdas[i];

		varying FVector3f Delta1, Delta2, Delta3, Delta4;
		const varying float DLambda = GetXPBDBendingElementDeltas(P1, P2, P3, P4, M1, M2, M3, M4, Dt, RestAngles[i], Lambda, StiffnessValue, Delta1, Delta2, Delta3, Delta4);

		Lambdas[i] = Lambda + DLambda;
		VectorScatter(&PandInvM[i1], SetVector4(P1 + Delta1, M1));
		VectorScatter(&PandInvM[i2], SetVector4(P2 + Delta2, M2));
		VectorScatter(&PandInvM[i3], SetVector4(P3 + Delta3, M3));
		VectorScatter(&PandInvM[i4], SetVector4(P4 + Delta4, M4));
	}
}

export void ApplyXPBDBendingConstraintsWithDamping(uniform FVector4f PandInvM[],
									const uniform FVector3f X1Array[],
									const uniform FVector3f X2Array[],
									const uniform FVector3f X3Array[],
									const uniform FVector3f X4Array[],
									const uniform int32 ConstraintIndex1[],
									const uniform int32 ConstraintIndex2[],
									const uniform int32 ConstraintIndex3[],
									const uniform int32 ConstraintIndex4[],
									const uniform float RestAngles[],
									const uniform bool IsBuckled[],
									uniform float Lambdas[],
									const uniform float Dt,
									const uniform float Stiffness,
									const uniform float BucklingStiffness,
									const uniform float DampingRatio,
									const uniform int32 NumConstraints)
{

	foreach(i = 0 ... NumConstraints)
	{
		const varying int32 i1 = ConstraintIndex1[i];
		const varying int32 i2 = ConstraintIndex2[i];
		const varying int32 i3 = ConstraintIndex3[i];
		const varying int32 i4 = ConstraintIndex4[i];
		
		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[i1]);
		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[i2]);
		const varying FVector4f PandInvM3 = VectorGather(&PandInvM[i3]);
		const varying FVector4f PandInvM4 = VectorGather(&PandInvM[i4]);

		varying FVector3f P1, P2, P3, P4;
		varying float M1, M2, M3, M4;

		UnzipPandInvM(PandInvM1, P1, M1);
		UnzipPandInvM(PandInvM2, P2, M2);
		UnzipPandInvM(PandInvM3, P3, M3);
		UnzipPandInvM(PandInvM4, P4, M4);
		
		const uniform int32 i0 = extract(i,0);

		const varying FVector3f X1 = VectorLoad(&X1Array[i0]);
		const varying FVector3f X2 = VectorLoad(&X2Array[i0]);
		const varying FVector3f X3 = VectorLoad(&X3Array[i0]);
		const varying FVector3f X4 = VectorLoad(&X4Array[i0]);
		
		const varying float StiffnessValue = select(IsBuckled[i], BucklingStiffness, Stiffness);
		const varying float Lambda = Lambdas[i];

		varying FVector3f Delta1, Delta2, Delta3, Delta4;
		const varying float DLambda = GetXPBDBendingElementDeltasWithDamping(P1, P2, P3, P4, X1, X2, X3, X4, M1, M2, M3, M4, Dt, RestAngles[i], Lambda, StiffnessValue, DampingRatio, Delta1, Delta2, Delta3, Delta4);

		Lambdas[i] = Lambda + DLambda;

		VectorScatter(&PandInvM[i1], SetVector4(P1 + Delta1, M1));
		VectorScatter(&PandInvM[i2], SetVector4(P2 + Delta2, M2));
		VectorScatter(&PandInvM[i3], SetVector4(P3 + Delta3, M3));
		VectorScatter(&PandInvM[i4], SetVector4(P4 + Delta4, M4));
	}
}

export void ApplyXPBDBendingDampingConstraints(uniform FVector4f PandInvM[],
									const uniform FVector3f X1Array[],
									const uniform FVector3f X2Array[],
									const uniform FVector3f X3Array[],
									const uniform FVector3f X4Array[],
									const uniform int32 ConstraintIndex1[],
									const uniform int32 ConstraintIndex2[],
									const uniform int32 ConstraintIndex3[],
									const uniform int32 ConstraintIndex4[],
									const uniform float RestAngles[],
									const uniform bool IsBuckled[],
									uniform float Lambdas[],
									const uniform float Dt,
									const uniform float Stiffness,
									const uniform float BucklingStiffness,
									const uniform float DampingRatio,
									const uniform int32 NumConstraints)
{

	foreach(i = 0 ... NumConstraints)
	{
		const varying int32 i1 = ConstraintIndex1[i];
		const varying int32 i2 = ConstraintIndex2[i];
		const varying int32 i3 = ConstraintIndex3[i];
		const varying int32 i4 = ConstraintIndex4[i];
		
		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[i1]);
		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[i2]);
		const varying FVector4f PandInvM3 = VectorGather(&PandInvM[i3]);
		const varying FVector4f PandInvM4 = VectorGather(&PandInvM[i4]);

		varying FVector3f P1, P2, P3, P4;
		varying float M1, M2, M3, M4;

		UnzipPandInvM(PandInvM1, P1, M1);
		UnzipPandInvM(PandInvM2, P2, M2);
		UnzipPandInvM(PandInvM3, P3, M3);
		UnzipPandInvM(PandInvM4, P4, M4);

		const uniform int32 i0 = extract(i,0);

		const varying FVector3f X1 = VectorLoad(&X1Array[i0]);
		const varying FVector3f X2 = VectorLoad(&X2Array[i0]);
		const varying FVector3f X3 = VectorLoad(&X3Array[i0]);
		const varying FVector3f X4 = VectorLoad(&X4Array[i0]);
		
		const varying float StiffnessValue = select(IsBuckled[i], BucklingStiffness, Stiffness);
		const varying float Lambda = Lambdas[i];

		varying FVector3f Delta1, Delta2, Delta3, Delta4;
		const varying float DLambda = GetXPBDBendingElementDampingDeltas(P1, P2, P3, P4, X1, X2, X3, X4, M1, M2, M3, M4, Dt, RestAngles[i], Lambda, StiffnessValue, DampingRatio, Delta1, Delta2, Delta3, Delta4);

		Lambdas[i] = Lambda + DLambda;
		VectorScatter(&PandInvM[i1], SetVector4(P1 + Delta1, M1));
		VectorScatter(&PandInvM[i2], SetVector4(P2 + Delta2, M2));
		VectorScatter(&PandInvM[i3], SetVector4(P3 + Delta3, M3));
		VectorScatter(&PandInvM[i4], SetVector4(P4 + Delta4, M4));
	}
}

export void ApplyXPBDBendingConstraintsWithMaps(uniform FVector4f PandInvM[],
											const uniform int32 ConstraintIndex1[],
											const uniform int32 ConstraintIndex2[],
											const uniform int32 ConstraintIndex3[],
											const uniform int32 ConstraintIndex4[],
											const uniform float RestAngles[],
											const uniform bool IsBuckled[],
											uniform float Lambdas[],
											const uniform float Dt,
											const uniform bool StiffnessHasMap,
											const uniform FVector2f& StiffnessOffsetRange,
											const uniform float StiffnessMapValues[],
											const uniform bool BucklingStiffnessHasMap,
											const uniform FVector2f& BucklingStiffnessOffsetRange,
											const uniform float BucklingStiffnessMapValues[],
											const uniform int32 NumConstraints)
{

	foreach(i = 0 ... NumConstraints)
	{
		const varying int32 i1 = ConstraintIndex1[i];
		const varying int32 i2 = ConstraintIndex2[i];
		const varying int32 i3 = ConstraintIndex3[i];
		const varying int32 i4 = ConstraintIndex4[i];

		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[i2]);
		const varying FVector4f PandInvM3 = VectorGather(&PandInvM[i3]);
		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[i1]);
		const varying FVector4f PandInvM4 = VectorGather(&PandInvM[i4]);

		varying FVector3f P1, P2, P3, P4;
		varying float M1, M2, M3, M4;

		UnzipPandInvM(PandInvM1, P1, M1);
		UnzipPandInvM(PandInvM2, P2, M2);
		UnzipPandInvM(PandInvM3, P3, M3);
		UnzipPandInvM(PandInvM4, P4, M4);
		
		const varying float Stiffness = StiffnessHasMap ? StiffnessOffsetRange.V[0] + StiffnessMapValues[i]*StiffnessOffsetRange.V[1] : StiffnessOffsetRange.V[0];
		const varying float BucklingStiffness = BucklingStiffnessHasMap ? BucklingStiffnessOffsetRange.V[0] + BucklingStiffnessMapValues[i]*BucklingStiffnessOffsetRange.V[1] : BucklingStiffnessOffsetRange.V[0];
		
		const varying float StiffnessValue = select(IsBuckled[i], BucklingStiffness, Stiffness);
		
		const varying float Lambda = Lambdas[i];

		varying FVector3f Delta1, Delta2, Delta3, Delta4;
		const varying float DLambda = GetXPBDBendingElementDeltas(P1, P2, P3, P4, M1, M2, M3, M4, Dt, RestAngles[i], Lambda, StiffnessValue, Delta1, Delta2, Delta3, Delta4);

		Lambdas[i] = Lambda + DLambda;

		VectorScatter(&PandInvM[i1], SetVector4(P1 + Delta1, M1));
		VectorScatter(&PandInvM[i2], SetVector4(P2 + Delta2, M2));
		VectorScatter(&PandInvM[i3], SetVector4(P3 + Delta3, M3));
		VectorScatter(&PandInvM[i4], SetVector4(P4 + Delta4, M4));
	}
}

export void ApplyXPBDBendingConstraintsWithDampingAndMaps(uniform FVector4f PandInvM[],
											const uniform FVector3f X1Array[],
											const uniform FVector3f X2Array[],
											const uniform FVector3f X3Array[],
											const uniform FVector3f X4Array[],
											const uniform int32 ConstraintIndex1[],
											const uniform int32 ConstraintIndex2[],
											const uniform int32 ConstraintIndex3[],
											const uniform int32 ConstraintIndex4[],
											const uniform float RestAngles[],
											const uniform bool IsBuckled[],
											uniform float Lambdas[],
											const uniform float Dt,
											const uniform bool StiffnessHasMap,
											const uniform FVector2f& StiffnessOffsetRange,
											const uniform float StiffnessMapValues[],
											const uniform bool BucklingStiffnessHasMap,
											const uniform FVector2f& BucklingStiffnessOffsetRange,
											const uniform float BucklingStiffnessMapValues[],
											const uniform bool DampingHasMap,
											const uniform FVector2f& DampingOffsetRange,
											const uniform float DampingMapValues[],
											const uniform int32 NumConstraints)
{

	foreach(i = 0 ... NumConstraints)
	{
		const varying int32 i1 = ConstraintIndex1[i];
		const varying int32 i2 = ConstraintIndex2[i];
		const varying int32 i3 = ConstraintIndex3[i];
		const varying int32 i4 = ConstraintIndex4[i];

		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[i2]);
		const varying FVector4f PandInvM3 = VectorGather(&PandInvM[i3]);
		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[i1]);
		const varying FVector4f PandInvM4 = VectorGather(&PandInvM[i4]);

		varying FVector3f P1, P2, P3, P4;
		varying float M1, M2, M3, M4;

		UnzipPandInvM(PandInvM1, P1, M1);
		UnzipPandInvM(PandInvM2, P2, M2);
		UnzipPandInvM(PandInvM3, P3, M3);
		UnzipPandInvM(PandInvM4, P4, M4);

		const varying float Stiffness = StiffnessHasMap ? StiffnessOffsetRange.V[0] + StiffnessMapValues[i]*StiffnessOffsetRange.V[1] : StiffnessOffsetRange.V[0];
		const varying float BucklingStiffness = BucklingStiffnessHasMap ? BucklingStiffnessOffsetRange.V[0] + BucklingStiffnessMapValues[i]*BucklingStiffnessOffsetRange.V[1] : BucklingStiffnessOffsetRange.V[0];
		const varying float DampingRatio = DampingHasMap ? DampingOffsetRange.V[0] + DampingMapValues[i]*DampingOffsetRange.V[1] : DampingOffsetRange.V[0];
		
		const varying float StiffnessValue = select(IsBuckled[i], BucklingStiffness, Stiffness);

		const uniform int32 i0 = extract(i,0);

		const varying FVector3f X1 = VectorLoad(&X1Array[i0]);
		const varying FVector3f X2 = VectorLoad(&X2Array[i0]);
		const varying FVector3f X3 = VectorLoad(&X3Array[i0]);
		const varying FVector3f X4 = VectorLoad(&X4Array[i0]);
		
		const varying float Lambda = Lambdas[i];

		varying FVector3f Delta1, Delta2, Delta3, Delta4;
		const varying float DLambda = GetXPBDBendingElementDeltasWithDamping(P1, P2, P3, P4, X1, X2, X3, X4, M1, M2, M3, M4, Dt, RestAngles[i], Lambda, StiffnessValue, DampingRatio, Delta1, Delta2, Delta3, Delta4);

		Lambdas[i] = Lambda + DLambda;

		VectorScatter(&PandInvM[i1], SetVector4(P1 + Delta1, M1));
		VectorScatter(&PandInvM[i2], SetVector4(P2 + Delta2, M2));
		VectorScatter(&PandInvM[i3], SetVector4(P3 + Delta3, M3));
		VectorScatter(&PandInvM[i4], SetVector4(P4 + Delta4, M4));
	}
}

export void ApplyXPBDBendingDampingConstraintsWithMaps(uniform FVector4f PandInvM[],
											const uniform FVector3f X1Array[],
											const uniform FVector3f X2Array[],
											const uniform FVector3f X3Array[],
											const uniform FVector3f X4Array[],
											const uniform int32 ConstraintIndex1[],
											const uniform int32 ConstraintIndex2[],
											const uniform int32 ConstraintIndex3[],
											const uniform int32 ConstraintIndex4[],
											const uniform float RestAngles[],
											const uniform bool IsBuckled[],
											uniform float Lambdas[],
											const uniform float Dt,
											const uniform bool StiffnessHasMap,
											const uniform FVector2f& StiffnessOffsetRange,
											const uniform float StiffnessMapValues[],
											const uniform bool BucklingStiffnessHasMap,
											const uniform FVector2f& BucklingStiffnessOffsetRange,
											const uniform float BucklingStiffnessMapValues[],
											const uniform bool DampingHasMap,
											const uniform FVector2f& DampingOffsetRange,
											const uniform float DampingMapValues[],
											const uniform int32 NumConstraints)
{

	foreach(i = 0 ... NumConstraints)
	{
		const varying int32 i1 = ConstraintIndex1[i];
		const varying int32 i2 = ConstraintIndex2[i];
		const varying int32 i3 = ConstraintIndex3[i];
		const varying int32 i4 = ConstraintIndex4[i];

		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[i2]);
		const varying FVector4f PandInvM3 = VectorGather(&PandInvM[i3]);
		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[i1]);
		const varying FVector4f PandInvM4 = VectorGather(&PandInvM[i4]);

		varying FVector3f P1, P2, P3, P4;
		varying float M1, M2, M3, M4;

		UnzipPandInvM(PandInvM1, P1, M1);
		UnzipPandInvM(PandInvM2, P2, M2);
		UnzipPandInvM(PandInvM3, P3, M3);
		UnzipPandInvM(PandInvM4, P4, M4);

		const varying float Stiffness = StiffnessHasMap ? StiffnessOffsetRange.V[0] + StiffnessMapValues[i]*StiffnessOffsetRange.V[1] : StiffnessOffsetRange.V[0];
		const varying float BucklingStiffness = BucklingStiffnessHasMap ? BucklingStiffnessOffsetRange.V[0] + BucklingStiffnessMapValues[i]*BucklingStiffnessOffsetRange.V[1] : BucklingStiffnessOffsetRange.V[0];
		const varying float DampingRatio = DampingHasMap ? DampingOffsetRange.V[0] + DampingMapValues[i]*DampingOffsetRange.V[1] : DampingOffsetRange.V[0];
		
		const varying float StiffnessValue = select(IsBuckled[i], BucklingStiffness, Stiffness);

		const uniform int32 i0 = extract(i,0);

		const varying FVector3f X1 = VectorLoad(&X1Array[i0]);
		const varying FVector3f X2 = VectorLoad(&X2Array[i0]);
		const varying FVector3f X3 = VectorLoad(&X3Array[i0]);
		const varying FVector3f X4 = VectorLoad(&X4Array[i0]);
		
		const varying float Lambda = Lambdas[i];

		varying FVector3f Delta1, Delta2, Delta3, Delta4;
		const varying float DLambda = GetXPBDBendingElementDampingDeltas(P1, P2, P3, P4, X1, X2, X3, X4, M1, M2, M3, M4, Dt, RestAngles[i], Lambda, StiffnessValue, DampingRatio, Delta1, Delta2, Delta3, Delta4);

		Lambdas[i] = Lambda + DLambda;
		
		VectorScatter(&PandInvM[i1], SetVector4(P1 + Delta1, M1));
		VectorScatter(&PandInvM[i2], SetVector4(P2 + Delta2, M2));
		VectorScatter(&PandInvM[i3], SetVector4(P3 + Delta3, M3));
		VectorScatter(&PandInvM[i4], SetVector4(P4 + Delta4, M4));
	}
}
