// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef __XPBDBENDING_ELEMENT_ISPH__
#define __XPBDBENDING_ELEMENT_ISPH__
#define EXPLICIT_VECTOR4 1

#include "Math/Vector.isph"
#include "Chaos/PBDSofts.isph"

inline FVector3f SafeDivide(const FVector3f& Numerator, const float Denominator)
{
	return VectorSelect(Denominator <= FLOAT_SMALL_NUMBER, FloatZeroVector, Numerator / Denominator);
}

inline varying float SafeDivide(const varying float Numerator, const varying float Denominator)
{
	return select(Denominator <= FLOAT_SMALL_NUMBER, (uniform float)0, Numerator / Denominator);
}

inline void CalcGradients(const FVector3f& P1, const FVector3f& P2, const FVector3f& P3, const FVector3f& P4, FVector3f& Grad1, FVector3f& Grad2, FVector3f& Grad3, FVector3f& Grad4)
{
	const FVector3f SharedEdgeNormalized = VectorGetSafeNormal(P2 - P1);
	const FVector3f P13CrossP23 = VectorCross(P1 - P3, P2 - P3);
	const float Normal1Len = VectorSize(P13CrossP23);
	const FVector3f Normal1 = SafeDivide(P13CrossP23, Normal1Len);
	const FVector3f P24CrossP14 = VectorCross(P2 - P4, P1 - P4);
	const float Normal2Len = VectorSize(P24CrossP14);
	const FVector3f Normal2 = SafeDivide(P24CrossP14, Normal2Len);

	const FVector3f N2CrossN1 = VectorCross(Normal2, Normal1);

	const float CosPhi = clamp(VectorDot(Normal1, Normal2), (uniform float)(-1), (uniform float)(1));
	const float SinPhi = clamp(VectorDot(N2CrossN1, SharedEdgeNormalized), (uniform float)(-1), (uniform float)(1));

	const FVector3f DPhiDN1_OverNormal1Len = SafeDivide(CosPhi * VectorCross(SharedEdgeNormalized, Normal2) - SinPhi * Normal2, Normal1Len);
	const FVector3f DPhiDN2_OverNormal2Len = SafeDivide(CosPhi * VectorCross(Normal1, SharedEdgeNormalized) - SinPhi * Normal1, Normal2Len);

	const FVector3f DPhiDP13 = VectorCross(P2 - P3, DPhiDN1_OverNormal1Len);
	const FVector3f DPhiDP23 = VectorCross(DPhiDN1_OverNormal1Len, P1 - P3);
	const FVector3f DPhiDP24 = VectorCross(P1 - P4, DPhiDN2_OverNormal2Len);
	const FVector3f DPhiDP14 = VectorCross(DPhiDN2_OverNormal2Len, P2 - P4);

	Grad1 = DPhiDP13 + DPhiDP14;
	Grad2 = DPhiDP23 + DPhiDP24;
	Grad3 = -1 * DPhiDP13 - DPhiDP23;
	Grad4 = -1 * DPhiDP14 - DPhiDP24;
}

// Returns Angle
inline float CalcGradientsAndAngle(const FVector3f& P1, const FVector3f& P2, const FVector3f& P3, const FVector3f& P4, FVector3f& Grad1, FVector3f& Grad2, FVector3f& Grad3, FVector3f& Grad4)
{
	const varying FVector3f SharedEdgeNormalized = VectorGetSafeNormal(P2 - P1);
	const varying FVector3f P13CrossP23 = VectorCross(P1 - P3, P2 - P3);
	const varying float Normal1Len = VectorSize(P13CrossP23);
	const varying FVector3f Normal1 = SafeDivide(P13CrossP23, Normal1Len);
	const varying FVector3f P24CrossP14 = VectorCross(P2 - P4, P1 - P4);
	const varying float Normal2Len = VectorSize(P24CrossP14);
	const varying FVector3f Normal2 = SafeDivide(P24CrossP14, Normal2Len);

	const varying FVector3f N2CrossN1 = VectorCross(Normal2, Normal1);

	const float CosPhi = clamp(VectorDot(Normal1, Normal2), (uniform float)(-1), (uniform float)(1));
	const float SinPhi = clamp(VectorDot(N2CrossN1, SharedEdgeNormalized), (uniform float)(-1), (uniform float)(1));
	const float Angle = atan2(SinPhi, CosPhi);

	const FVector3f DPhiDN1_OverNormal1Len = SafeDivide(CosPhi * VectorCross(SharedEdgeNormalized, Normal2) - SinPhi * Normal2, Normal1Len);
	const FVector3f DPhiDN2_OverNormal2Len = SafeDivide(CosPhi * VectorCross(Normal1, SharedEdgeNormalized) - SinPhi * Normal1, Normal2Len);

	const FVector3f DPhiDP13 = VectorCross(P2 - P3, DPhiDN1_OverNormal1Len);
	const FVector3f DPhiDP23 = VectorCross(DPhiDN1_OverNormal1Len, P1 - P3);
	const FVector3f DPhiDP24 = VectorCross(P1 - P4, DPhiDN2_OverNormal2Len);
	const FVector3f DPhiDP14 = VectorCross(DPhiDN2_OverNormal2Len, P2 - P4);

	Grad1 = DPhiDP13 + DPhiDP14;
	Grad2 = DPhiDP23 + DPhiDP24;
	Grad3 = -1 * DPhiDP13 - DPhiDP23;
	Grad4 = -1 * DPhiDP14 - DPhiDP24;

	return Angle;
}

// Elastic part only
inline float GetXPBDBendingElementDeltas(const FVector3f& P1, const FVector3f& P2, const FVector3f& P3, const FVector3f& P4,
	const varying float InvM1, const varying float InvM2, const varying float InvM3, const varying float InvM4,
	const uniform float Dt, const varying float RestAngle, const varying float Lambda, const varying float StiffnessValue,
	FVector3f& Delta1, FVector3f& Delta2, FVector3f& Delta3, FVector3f& Delta4)
{
	varying FVector3f Grad1, Grad2, Grad3, Grad4;
	const varying float Angle = CalcGradientsAndAngle(P1, P2, P3, P4, Grad1, Grad2, Grad3, Grad4);
	
	const varying float AlphaInv  = StiffnessValue * Dt * Dt;	
	const varying float ElasticTerm = AlphaInv * (Angle - RestAngle);
	const varying float Denom = AlphaInv*(InvM1 * VectorSizeSquared(Grad1) + InvM2 * VectorSizeSquared(Grad2) + InvM3 * VectorSizeSquared(Grad3) + InvM4 * VectorSizeSquared(Grad4)) + 1.f;

	const varying float DLambda = (-ElasticTerm - Lambda) / Denom;

	Delta1 = DLambda * InvM1 * Grad1;
	Delta2 = DLambda * InvM2 * Grad2;
	Delta3 = DLambda * InvM3 * Grad3;
	Delta4 = DLambda * InvM4 * Grad4;
	return DLambda;
}

// This is a following the original XPBD paper using a single lambda for stretch and damping.
template <typename DampingFloatType>
inline float GetXPBDBendingElementDeltasWithDamping(const FVector3f& P1, const FVector3f& P2, const FVector3f& P3, const FVector3f& P4,
	const FVector3f& X1, const FVector3f& X2, const FVector3f& X3, const FVector3f& X4,
	const varying float InvM1, const varying float InvM2, const varying float InvM3, const varying float InvM4,
	const uniform float Dt, const varying float RestAngle, const varying float Lambda, 
	const varying float StiffnessValue, const DampingFloatType DampingRatio,
	FVector3f& Delta1, FVector3f& Delta2, FVector3f& Delta3, FVector3f& Delta4)
{
	varying FVector3f Grad1, Grad2, Grad3, Grad4;
	const varying float Angle = CalcGradientsAndAngle(P1, P2, P3, P4, Grad1, Grad2, Grad3, Grad4);
	
	const varying float AlphaInv = StiffnessValue * Dt * Dt;
	const varying float CombinedInvMass = InvM1 + InvM2 + InvM3 + InvM4;
	const varying float Damping = 2.f * DampingRatio * sqrt(StiffnessValue / CombinedInvMass);
	const varying float BetaDt = Damping * Dt;
	
	const varying FVector3f V1TimesDt = P1 - X1;
	const varying FVector3f V2TimesDt = P2 - X2;
	const varying FVector3f V3TimesDt = P3 - X3;
	const varying FVector3f V4TimesDt = P4 - X4;
	const varying float DampingTerm = BetaDt * (VectorDot(V1TimesDt, Grad1) + VectorDot(V2TimesDt, Grad2) + VectorDot(V3TimesDt, Grad3) + VectorDot(V4TimesDt, Grad4));
	const varying float ElasticTerm = AlphaInv * (Angle - RestAngle);

	const varying float Denom = (AlphaInv + BetaDt)*(InvM1 * VectorSizeSquared(Grad1) + InvM2 * VectorSizeSquared(Grad2) + InvM3 * VectorSizeSquared(Grad3) + InvM4 * VectorSizeSquared(Grad4)) + 1.f;

	const varying float DLambda = (-ElasticTerm - DampingTerm - Lambda) / Denom;

	Delta1 = DLambda * InvM1 * Grad1;
	Delta2 = DLambda * InvM2 * Grad2;
	Delta3 = DLambda * InvM3 * Grad3;
	Delta4 = DLambda * InvM4 * Grad4;
	return DLambda;
}

// Damping only (separate from elastic part)
template <typename DampingFloatType>
inline float GetXPBDBendingElementDampingDeltas(const FVector3f& P1, const FVector3f& P2, const FVector3f& P3, const FVector3f& P4,
	const FVector3f& X1, const FVector3f& X2, const FVector3f& X3, const FVector3f& X4,
	const varying float InvM1, const varying float InvM2, const varying float InvM3, const varying float InvM4,
	const uniform float Dt, const varying float RestAngle, const varying float Lambda, 
	const varying float StiffnessValue, const DampingFloatType DampingRatio,
	FVector3f& Delta1, FVector3f& Delta2, FVector3f& Delta3, FVector3f& Delta4)
{
	varying FVector3f Grad1, Grad2, Grad3, Grad4;
	CalcGradients(P1, P2, P3, P4, Grad1, Grad2, Grad3, Grad4);
	
	const varying float CombinedInvMass = InvM1 + InvM2 + InvM3 + InvM4;
	const varying float Damping = 2.f * DampingRatio * sqrt(StiffnessValue / CombinedInvMass);
	const varying float BetaDt = Damping * Dt;
	
	const varying FVector3f V1TimesDt = P1 - X1;
	const varying FVector3f V2TimesDt = P2 - X2;
	const varying FVector3f V3TimesDt = P3 - X3;
	const varying FVector3f V4TimesDt = P4 - X4;
	const varying float DampingTerm = BetaDt * (VectorDot(V1TimesDt, Grad1) + VectorDot(V2TimesDt, Grad2) + VectorDot(V3TimesDt, Grad3) + VectorDot(V4TimesDt, Grad4));

	const varying float Denom = BetaDt*(InvM1 * VectorSizeSquared(Grad1) + InvM2 * VectorSizeSquared(Grad2) + InvM3 * VectorSizeSquared(Grad3) + InvM4 * VectorSizeSquared(Grad4)) + 1.f;

	const varying float DLambda = ( -DampingTerm - Lambda) / Denom;

	Delta1 = DLambda * InvM1 * Grad1;
	Delta2 = DLambda * InvM2 * Grad2;
	Delta3 = DLambda * InvM3 * Grad3;
	Delta4 = DLambda * InvM4 * Grad4;
	return DLambda;
}

#endif