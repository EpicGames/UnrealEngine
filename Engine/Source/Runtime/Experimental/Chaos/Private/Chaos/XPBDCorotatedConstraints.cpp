// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/XPBDCorotatedConstraints.h"

namespace Chaos::Softs
{

FXPBDCorotatedTetrahedralConstraints::FXPBDCorotatedTetrahedralConstraints(
	const FSolverParticles& InParticles,
	const TArray<TVector<int32, 4>>& InMesh,
	const bool bRecordMetricIn,
	const FSolverReal& EMesh,
	const FSolverReal& NuMesh)
	: bRecordMetric(bRecordMetricIn), MeshConstraints(InMesh)
{
	LambdaArray.Init((FSolverReal)0., 2 * MeshConstraints.Num());
	DmInverse.Init((FSolverReal)0., 9 * MeshConstraints.Num());
	Measure.Init((FSolverReal)0., MeshConstraints.Num());
	Lambda = EMesh * NuMesh / (((FSolverReal)1. + NuMesh) * ((FSolverReal)1. - (FSolverReal)2. * NuMesh));
	Mu = EMesh / ((FSolverReal)2. * ((FSolverReal)1. + NuMesh));
	LambdaElementArray.Init(Lambda, MeshConstraints.Num());
	MuElementArray.Init(Mu, MeshConstraints.Num());
	for (int32 ElemIdx = 0; ElemIdx < InMesh.Num(); ElemIdx++)
	{
		const PMatrix<FSolverReal, 3, 3> Dm = DsInit(ElemIdx, InParticles);
		const PMatrix<FSolverReal, 3, 3> DmInv = Dm.Inverse();
		for (int32 Row = 0; Row < 3; Row++)
		{
			for (int32 Col = 0; Col < 3; Col++)
			{
				DmInverse[(3 * 3) * ElemIdx + 3 * Row + Col] = DmInv.GetAt(Row, Col);
			}
		}

		Measure[ElemIdx] = Dm.Determinant() / (FSolverReal)6.;

		if (Measure[ElemIdx] < (FSolverReal)0.)
		{
			Measure[ElemIdx] = -Measure[ElemIdx];
		}
	}
	DmInverseSave = DmInverse;
	InitColor(InParticles);
}

FXPBDCorotatedTetrahedralConstraints::FXPBDCorotatedTetrahedralConstraints(
	const FSolverParticles& InParticles,
	const TArray<TVector<int32, 4>>& InMesh,
	const TArray<FSolverReal>& EMeshArray,
	const FSolverReal& NuMesh,
	const bool bRecordMetricIn)
	: bRecordMetric(bRecordMetricIn), MeshConstraints(InMesh)
{
	ensureMsgf(EMeshArray.Num() == InMesh.Num(), TEXT("Input Young Modulus Array Size is wrong"));
	LambdaArray.Init((FSolverReal)0., 2 * MeshConstraints.Num());
	DmInverse.Init((FSolverReal)0., 9 * MeshConstraints.Num());
	Measure.Init((FSolverReal)0., MeshConstraints.Num());
	LambdaElementArray.Init((FSolverReal)0., MeshConstraints.Num());
	MuElementArray.Init((FSolverReal)0., MeshConstraints.Num());

	for (int32 ElemIdx = 0; ElemIdx < InMesh.Num(); ElemIdx++)
	{
		for (int32 TetLocalIdx = 0; TetLocalIdx < 4; TetLocalIdx++)
		{
			ensure(MeshConstraints[ElemIdx][TetLocalIdx] > -1 && MeshConstraints[ElemIdx][TetLocalIdx] < int32(InParticles.Size()));
		}
	}

	for (int32 ElemIdx = 0; ElemIdx < InMesh.Num(); ElemIdx++)
	{
		LambdaElementArray[ElemIdx] = EMeshArray[ElemIdx] * NuMesh / (((FSolverReal)1. + NuMesh) * ((FSolverReal)1. - (FSolverReal)2. * NuMesh));
		MuElementArray[ElemIdx] = EMeshArray[ElemIdx] / ((FSolverReal)2. * ((FSolverReal)1. + NuMesh));

		const PMatrix<FSolverReal, 3, 3> Dm = DsInit(ElemIdx, InParticles);
		const PMatrix<FSolverReal, 3, 3> DmInv = Dm.Inverse();
		for (int32 Row = 0; Row < 3; Row++)
		{
			for (int32 Col = 0; Col < 3; Col++)
			{
				DmInverse[(3 * 3) * ElemIdx + 3 * Row + Col] = DmInv.GetAt(Row, Col);
			}
		}

		Measure[ElemIdx] = Dm.Determinant() / (FSolverReal)6.;

		if (Measure[ElemIdx] < (FSolverReal)0.)
		{
			Measure[ElemIdx] = -Measure[ElemIdx];
		}
	}
	DmInverseSave = DmInverse;
	InitColor(InParticles);
}

FXPBDCorotatedTetrahedralConstraints::FXPBDCorotatedTetrahedralConstraints(
	const FSolverParticlesRange& InParticles,
	const TArray<TVector<int32, 4>>& InMesh,
	const TArray<FSolverReal>& EMeshArray,
	const TArray<FSolverReal>& NuMeshArray,
	TArray<FSolverReal>&& AlphaJMeshArray,
	const FDeformableXPBDCorotatedParams& InParams,
	const FSolverReal& NuMesh,
	const bool bRecordMetricIn,
	const bool bDoColoring)
	: CorotatedParams(InParams), AlphaJArray(MoveTemp(AlphaJMeshArray)), bRecordMetric(bRecordMetricIn), MeshConstraints(InMesh)
{
	ensureMsgf(EMeshArray.Num() == InMesh.Num(), TEXT("Input Young Modulus Array Size is wrong"));
	LambdaArray.Init((FSolverReal)0., 2 * MeshConstraints.Num());
	DmInverse.Init((FSolverReal)0., 9 * MeshConstraints.Num());
	Measure.Init((FSolverReal)0., MeshConstraints.Num());
	LambdaElementArray.Init((FSolverReal)0., MeshConstraints.Num());
	MuElementArray.Init((FSolverReal)0., MeshConstraints.Num());

	for (int32 ElemIdx = 0; ElemIdx < InMesh.Num(); ElemIdx++)
	{
		for (int32 TetLocalIdx = 0; TetLocalIdx < 4; TetLocalIdx++)
		{
			ensure(MeshConstraints[ElemIdx][TetLocalIdx] > -1 && MeshConstraints[ElemIdx][TetLocalIdx] < int32(InParticles.Size()));

			if (InParticles.InvM(MeshConstraints[ElemIdx][TetLocalIdx]) == (FSolverReal)0.)
			{
				AlphaJArray[ElemIdx] = (FSolverReal)1.;
			}
		}
	}

	for (int32 ElemIdx = 0; ElemIdx < InMesh.Num(); ElemIdx++)
	{
		LambdaElementArray[ElemIdx] = EMeshArray[ElemIdx] * NuMeshArray[ElemIdx] / (((FSolverReal)1. + NuMeshArray[ElemIdx]) * ((FSolverReal)1. - (FSolverReal)2. * NuMeshArray[ElemIdx]));
		MuElementArray[ElemIdx] = EMeshArray[ElemIdx] / ((FSolverReal)2. * ((FSolverReal)1. + NuMeshArray[ElemIdx]));

		const PMatrix<FSolverReal, 3, 3> Dm = DsInit(ElemIdx, InParticles);
		const PMatrix<FSolverReal, 3, 3> DmInv = Dm.Inverse();
		for (int32 Row = 0; Row < 3; Row++)
		{
			for (int32 Col = 0; Col < 3; Col++)
			{
				DmInverse[(3 * 3) * ElemIdx + 3 * Row + Col] = DmInv.GetAt(Row, Col);
			}
		}

		Measure[ElemIdx] = Dm.Determinant() / (FSolverReal)6.;

		if (Measure[ElemIdx] < (FSolverReal)0.)
		{
			Measure[ElemIdx] = -Measure[ElemIdx];
		}
	}
	DmInverseSave = DmInverse;
	if (bDoColoring)
	{
		InitColor(InParticles);
	}
}

FXPBDCorotatedTetrahedralConstraints::FXPBDCorotatedTetrahedralConstraints(
	const FSolverParticles& InParticles,
	const TArray<TVector<int32, 4>>& InMesh,
	const FSolverReal GridN,
	const FSolverReal& EMesh,
	const FSolverReal& NuMesh)
	: MeshConstraints(InMesh)
{
	LambdaArray.Init((FSolverReal)0., 2 * MeshConstraints.Num());
	DmInverse.Init((FSolverReal)0., 9 * MeshConstraints.Num());
	Measure.Init((FSolverReal)0., MeshConstraints.Num());
	Lambda = EMesh * NuMesh / (((FSolverReal)1. + NuMesh) * ((FSolverReal)1. - (FSolverReal)2. * NuMesh));
	Mu = EMesh / ((FSolverReal)2. * ((FSolverReal)1. + NuMesh));
	for (int32 ElemIdx = 0; ElemIdx < InMesh.Num(); ElemIdx++)
	{
		const PMatrix<FSolverReal, 3, 3> Dm = DsInit(ElemIdx, InParticles);
		const PMatrix<FSolverReal, 3, 3> DmInv = Dm.Inverse();
		for (int32 Row = 0; Row < 3; Row++)
		{
			for (int32 Col = 0; Col < 3; Col++)
			{
				DmInverse[(3 * 3) * ElemIdx + 3 * Row + Col] = DmInv.GetAt(Row, Col);
			}
		}

		Measure[ElemIdx] = Dm.Determinant() / (FSolverReal)6.;

		if (Measure[ElemIdx] < (FSolverReal)0.)
		{
			Measure[ElemIdx] = -Measure[ElemIdx];
		}
	}
	DmInverseSave = DmInverse;
}

PMatrix<FSolverReal, 3, 3> FXPBDCorotatedTetrahedralConstraints::DsInit(const int32 ElemIdx, const FSolverParticlesRange& InParticles) const
{
	PMatrix<FSolverReal, 3, 3> Result((FSolverReal)0.);
	for (int32 I = 0; I < 3; I++)
	{
		for (int32 Col = 0; Col < 3; Col++)
		{
			Result.SetAt(Col, I, InParticles.GetX(MeshConstraints[ElemIdx][I + 1])[Col] - InParticles.GetX(MeshConstraints[ElemIdx][0])[Col]);
		}
	}
	return Result;
}

PMatrix<FSolverReal, 3, 3> FXPBDCorotatedTetrahedralConstraints::Ds(const int32 ElemIdx, const FSolverParticlesRange& InParticles) const
{
	PMatrix<FSolverReal, 3, 3> Result((FSolverReal)0.);
	for (int32 I = 0; I < 3; I++)
	{
		for (int32 Col = 0; Col < 3; Col++)
		{
			Result.SetAt(Col, I, InParticles.GetP(MeshConstraints[ElemIdx][I+1])[Col] - InParticles.GetP(MeshConstraints[ElemIdx][0])[Col]);
		}
	}
	return Result;
}

void FXPBDCorotatedTetrahedralConstraints::ApplyInSerial(FSolverParticlesRange& Particles, const FSolverReal Dt, const int32 ElementIndex) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(STAT_ChaosXPBDCorotatedApplySingle);

	const TVec4<TVector<FSolverReal, 3>> PolarDelta = GetPolarDelta(Particles, Dt, ElementIndex);

	for (int32 Index = 0; Index < 4; Index++)
	{
		Particles.P(MeshConstraints[ElementIndex][Index]) += PolarDelta[Index];
	}

	const TVec4<TVector<FSolverReal, 3>> DetDelta = GetDeterminantDelta(Particles, Dt, ElementIndex);

	for (int32 Index = 0; Index < 4; Index++)
	{
		Particles.P(MeshConstraints[ElementIndex][Index]) += DetDelta[Index];
	}
}

void FXPBDCorotatedTetrahedralConstraints::ApplyInSerial(FSolverParticlesRange& Particles, const FSolverReal Dt) const
{
	SCOPE_CYCLE_COUNTER(STAT_ChaosXPBDCorotated);
	TRACE_CPUPROFILER_EVENT_SCOPE(STAT_ChaosXPBDCorotatedApplySerial);
	const int32 ConstraintColorNum = ConstraintsPerColorStartIndex.Num() - 1;
	for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
	{
		const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
		const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
		for (int32 Index = 0; Index < ColorSize; Index++)
		{
			const int32 ConstraintIndex = ColorStart + Index;
			ApplyInSerial(Particles, Dt, ConstraintIndex);
		}
	}
}

void FXPBDCorotatedTetrahedralConstraints::ApplyInParallel(FSolverParticlesRange& Particles, const FSolverReal Dt) const
{
	if (bRecordMetric)
	{
		GError.Init((FSolverReal)0., 3 * Particles.Size());
		HErrorArray.Init((FSolverReal)0., 2 * MeshConstraints.Num());
	}

	{
		SCOPE_CYCLE_COUNTER(STAT_ChaosXPBDCorotated);
		TRACE_CPUPROFILER_EVENT_SCOPE(STAT_ChaosXPBDCorotatedApply);
		if ((ConstraintsPerColorStartIndex.Num() > 1))
		{
			const int32 ConstraintColorNum = ConstraintsPerColorStartIndex.Num() - 1;

			for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
			{
				const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
				const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;

				int32 NumBatch = ColorSize / CorotatedParams.XPBDCorotatedBatchSize;
				if (ColorSize % CorotatedParams.XPBDCorotatedBatchSize != 0)
				{
					NumBatch += 1;
				}

				PhysicsParallelFor(NumBatch, [&](const int32 BatchIndex)
					{
						for (int32 BatchSubIndex = 0; BatchSubIndex < CorotatedParams.XPBDCorotatedBatchSize; BatchSubIndex++)
						{
							const int32 TaskIndex = CorotatedParams.XPBDCorotatedBatchSize * BatchIndex + BatchSubIndex;
							const int32 ConstraintIndex = ColorStart + TaskIndex;
							if (ConstraintIndex < ColorStart + ColorSize)
							{
								ApplyInSerial(Particles, Dt, ConstraintIndex);
							}
						}
					}, NumBatch < CorotatedParams.XPBDCorotatedBatchThreshold);
			}
		}
	}
}

void FXPBDCorotatedTetrahedralConstraints::ApplyInParallel(FSolverParticles& Particles, const FSolverReal Dt) const
{
	FSolverParticlesRange Range(Particles);
	ApplyInParallel(Range, Dt);
}

TVec4<TVector<FSolverReal, 3>> FXPBDCorotatedTetrahedralConstraints::GetPolarGradient(const PMatrix<FSolverReal, 3, 3>& Fe, const PMatrix<FSolverReal, 3, 3>& Re, const PMatrix<FSolverReal, 3, 3>& DmInvT, const FSolverReal C1) const
{
	TVec4<TVector<FSolverReal, 3>> DeltaC1(TVector<FSolverReal, 3>((FSolverReal)0.));
	const PMatrix<FSolverReal, 3, 3> A = DmInvT * (Fe - Re);
	for (int32 Alpha = 0; Alpha < 3; Alpha++)
	{
		for (int32 Col = 0; Col < 3; Col++)
		{
			DeltaC1[0][Alpha] -= A.GetAt(Alpha, Col);
		}
	}
	for (int32 TetLocalIdx = 0; TetLocalIdx < 3; TetLocalIdx++)
	{
		for (int32 Alpha = 0; Alpha < 3; Alpha++)
		{
			DeltaC1[TetLocalIdx + 1][Alpha] = A.GetAt(Alpha, TetLocalIdx);
		}
	}

	if (C1 != 0)
	{
		for (int32 Index = 0; Index < 4; Index++)
		{
			for (int32 Alpha = 0; Alpha < 3; Alpha++)
			{
				DeltaC1[Index][Alpha] /= C1;
			}
		}
	}
	return DeltaC1;
}

TVec4<TVector<FSolverReal, 3>> FXPBDCorotatedTetrahedralConstraints::GetDeterminantGradient(const PMatrix<FSolverReal, 3, 3>& Fe, const PMatrix<FSolverReal, 3, 3>& DmInvT) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(STAT_ChaosXPBDCorotatedGetDetGradient);
	TVec4<TVector<FSolverReal, 3>> DeltaC2(TVector<FSolverReal, 3>((FSolverReal)0.));

	PMatrix<FSolverReal, 3, 3> JFinvT((FSolverReal)0.);
	JFinvT.SetAt(0, 0, Fe.GetAt(1, 1) * Fe.GetAt(2, 2) - Fe.GetAt(2, 1) * Fe.GetAt(1, 2));
	JFinvT.SetAt(0, 1, Fe.GetAt(2, 0) * Fe.GetAt(1, 2) - Fe.GetAt(1, 0) * Fe.GetAt(2, 2));
	JFinvT.SetAt(0, 2, Fe.GetAt(1, 0) * Fe.GetAt(2, 1) - Fe.GetAt(2, 0) * Fe.GetAt(1, 1));
	JFinvT.SetAt(1, 0, Fe.GetAt(2, 1) * Fe.GetAt(0, 2) - Fe.GetAt(0, 1) * Fe.GetAt(2, 2));
	JFinvT.SetAt(1, 1, Fe.GetAt(0, 0) * Fe.GetAt(2, 2) - Fe.GetAt(2, 0) * Fe.GetAt(0, 2));
	JFinvT.SetAt(1, 2, Fe.GetAt(2, 0) * Fe.GetAt(0, 1) - Fe.GetAt(0, 0) * Fe.GetAt(2, 1));
	JFinvT.SetAt(2, 0, Fe.GetAt(0, 1) * Fe.GetAt(1, 2) - Fe.GetAt(1, 1) * Fe.GetAt(0, 2));
	JFinvT.SetAt(2, 1, Fe.GetAt(1, 0) * Fe.GetAt(0, 2) - Fe.GetAt(0, 0) * Fe.GetAt(1, 2));
	JFinvT.SetAt(2, 2, Fe.GetAt(0, 0) * Fe.GetAt(1, 1) - Fe.GetAt(1, 0) * Fe.GetAt(0, 1));

	const PMatrix<FSolverReal, 3, 3> JinvTDmInvT = DmInvT * JFinvT;

	for (int32 TetLocalIdx = 0; TetLocalIdx < 3; TetLocalIdx++)
	{
		for (int32 Alpha = 0; Alpha < 3; Alpha++)
		{
			DeltaC2[TetLocalIdx + 1][Alpha] = JinvTDmInvT.GetAt(Alpha, TetLocalIdx);
		}
	}
	for (int32 Alpha = 0; Alpha < 3; Alpha++)
	{
		for (int32 Col = 0; Col < 3; Col++)
		{
			DeltaC2[0][Alpha] -= JinvTDmInvT.GetAt(Alpha, Col);
		}
	}
	return DeltaC2;
}

void FXPBDCorotatedTetrahedralConstraints::ModifyDmInverseFromMuscleLength(const int32 ElemIdx, const FSolverReal FiberLengthRatio, const PMatrix<FSolverReal, 3, 3>& FiberDir, const FSolverReal ContractionVolumeScale) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(STAT_ChaosXPBDCorotatedModifyDmInverseFromMuscleLength);
	PMatrix<FSolverReal, 3, 3> DmInv = ElementDmInvSave(ElemIdx);
	if (FiberLengthRatio < 1)
	{
		const PMatrix<FSolverReal, 3, 3> D(1 / FiberLengthRatio, FMath::Pow(FiberLengthRatio, ContractionVolumeScale/FSolverReal(2)), FMath::Pow(FiberLengthRatio, ContractionVolumeScale/FSolverReal(2)));
		const PMatrix<FSolverReal, 3, 3> Factor = FiberDir * D * FiberDir.GetTransposed();
		DmInv = Factor * DmInv;
	}
	for (int32 Row = 0; Row < 3; Row++)
	{
		for (int32 Column = 0; Column < 3; Column++)
		{
			DmInverse[(3 * 3) * ElemIdx + 3 * Row + Column] = DmInv.GetAt(Row, Column);
		}
	}
}

void FXPBDCorotatedTetrahedralConstraints::ModifyDmInverseSaveFromInflationVolumeScale(const int32 ElemIdx, const FSolverReal InflationVolumeScale, const PMatrix<FSolverReal, 3, 3>& FiberDir)
{
	PMatrix<FSolverReal, 3, 3> DmInv = ElementDmInvSave(ElemIdx);
	if (FMath::Abs(InflationVolumeScale - 1) > UE_SMALL_NUMBER)
	{
		const FSolverReal InvSqrt = FMath::InvSqrt(InflationVolumeScale);
		const PMatrix<FSolverReal, 3, 3> D(1, InvSqrt, InvSqrt);
		const PMatrix<FSolverReal, 3, 3> Factor = FiberDir * D * FiberDir.GetTransposed();
		DmInv = Factor * DmInv;
	}
	for (int32 Row = 0; Row < 3; Row++)
	{
		for (int32 Column = 0; Column < 3; Column++)
		{
			DmInverseSave[(3 * 3) * ElemIdx + 3 * Row + Column] = DmInv.GetAt(Row, Column);
			DmInverse[(3 * 3) * ElemIdx + 3 * Row + Column] = DmInv.GetAt(Row, Column);
		}
	}
}

void FXPBDCorotatedTetrahedralConstraints::InitColor(const FSolverParticlesRange& Particles)
{
	const TArray<TArray<int32>> ConstraintsPerColor = FGraphColoring::ComputeGraphColoringParticlesOrRange(MeshConstraints, Particles, 0, Particles.Size());

	TArray<TVec4<int32>> ReorderedConstraints;
	TArray<FSolverReal> ReorderedMeasure;
	TArray<FSolverReal> ReorderedDmInverse;
	TArray<int32> OrigToReorderedIndices;
	ReorderedConstraints.SetNumUninitialized(MeshConstraints.Num());
	ReorderedMeasure.SetNumUninitialized(Measure.Num());
	ReorderedDmInverse.SetNumUninitialized(DmInverse.Num());
	OrigToReorderedIndices.SetNumUninitialized(MeshConstraints.Num());

	ConstraintsPerColorStartIndex.Reset(ConstraintsPerColor.Num() + 1);

	int32 ReorderedIndex = 0;
	for (const TArray<int32>& ConstraintsBatch : ConstraintsPerColor)
	{
		ConstraintsPerColorStartIndex.Add(ReorderedIndex);
		for (const int32& BatchConstraint : ConstraintsBatch)
		{
			const int32 OrigIndex = BatchConstraint;
			ReorderedConstraints[ReorderedIndex] = MeshConstraints[OrigIndex];
			ReorderedMeasure[ReorderedIndex] = Measure[OrigIndex];
			for (int32 FlatIdx = 0; FlatIdx < 9; FlatIdx++)
			{
				ReorderedDmInverse[9 * ReorderedIndex + FlatIdx] = DmInverse[9 * OrigIndex + FlatIdx];
			}
			OrigToReorderedIndices[OrigIndex] = ReorderedIndex;

			++ReorderedIndex;
		}
	}
	ConstraintsPerColorStartIndex.Add(ReorderedIndex);

	MeshConstraints = MoveTemp(ReorderedConstraints);
	Measure = MoveTemp(ReorderedMeasure);
	DmInverse = MoveTemp(ReorderedDmInverse);
}

TVec4<TVector<FSolverReal, 3>> FXPBDCorotatedTetrahedralConstraints::GetDeterminantDelta(const FSolverParticlesRange& Particles, const FSolverReal Dt, const int32 ElementIndex, const FSolverReal Tol) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(STAT_ChaosXPBDCorotatedApplyDet);

	const PMatrix<FSolverReal, 3, 3> Fe = F(ElementIndex, Particles);
	const PMatrix<FSolverReal, 3, 3> DmInvT = ElementDmInv(ElementIndex).GetTransposed();

	const FSolverReal J = Fe.Determinant();
	if (J - AlphaJArray[ElementIndex] < Tol)
	{
		return TVec4<TVector<FSolverReal, 3>>(TVector<FSolverReal, 3>((FSolverReal)0.));
	}

	const TVec4<TVector<FSolverReal, 3>> DeltaC2 = GetDeterminantGradient(Fe, DmInvT);

	FSolverReal AlphaTilde = (FSolverReal)2. / (Dt * Dt * LambdaElementArray[ElementIndex] * Measure[ElementIndex]);

	if (LambdaElementArray[ElementIndex] > (FSolverReal)1. / (FSolverReal)UE_SMALL_NUMBER)
	{
		AlphaTilde = (FSolverReal)0.;
	}

	if (bRecordMetric)
	{
		HError += J - 1 + AlphaTilde * LambdaArray[2 * ElementIndex + 1];
		for (int32 TetLocalIdx = 0; TetLocalIdx < 4; TetLocalIdx++)
		{
			for (int32 Alpha = 0; Alpha < 3; Alpha++)
			{
				GError[MeshConstraints[ElementIndex][TetLocalIdx] * 3 + Alpha] -= DeltaC2[TetLocalIdx][Alpha] * LambdaArray[2 * ElementIndex + 1];
			}
		}
		HErrorArray[2 * ElementIndex + 1] = J - 1 + AlphaTilde * LambdaArray[2 * ElementIndex + 1];
	}

	FSolverReal DLambda = (AlphaJArray[ElementIndex] - J) - AlphaTilde * LambdaArray[2 * ElementIndex + 1];

	FSolverReal Denom = AlphaTilde;
	for (int32 Index = 0; Index < 4; Index++)
	{
		for (int32 Alpha = 0; Alpha < 3; Alpha++)
		{
			Denom += DeltaC2[Index][Alpha] * Particles.InvM(MeshConstraints[ElementIndex][Index]) * DeltaC2[Index][Alpha];
		}
	}
	DLambda /= Denom;
	LambdaArray[2 * ElementIndex + 1] += DLambda;
	TVec4<TVector<FSolverReal, 3>> Delta(TVector<FSolverReal, 3>((FSolverReal)0.));
	for (int32 Index = 0; Index < 4; Index++)
	{
		for (int32 Alpha = 0; Alpha < 3; Alpha++)
		{
			Delta[Index][Alpha] = Particles.InvM(MeshConstraints[ElementIndex][Index]) * DeltaC2[Index][Alpha] * DLambda;
		}
	}
	return Delta;
}

TVec4<TVector<FSolverReal, 3>> FXPBDCorotatedTetrahedralConstraints::GetPolarDelta(const FSolverParticlesRange& Particles, const FSolverReal Dt, const int32 ElementIndex, const FSolverReal Tol) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(STAT_ChaosXPBDCorotatedApplyPolar);
	SCOPE_CYCLE_COUNTER(STAT_ChaosXPBDCorotatedPolar);
	const PMatrix<FSolverReal, 3, 3> Fe = F(ElementIndex, Particles);

	PMatrix<FSolverReal, 3, 3> Re((FSolverReal)0.), Se((FSolverReal)0.);

	Chaos::PolarDecomposition(Fe, Re, Se);

	Re *= FGenericPlatformMath::Pow(AlphaJArray[ElementIndex], (FSolverReal)1. / (FSolverReal)3.);

	FSolverReal C1 = (FSolverReal)0.;
	for (int32 I = 0; I < 3; I++)
	{
		for (int32 J = 0; J < 3; J++)
		{
			C1 += FMath::Square((Fe - Re).GetAt(I, J));
		}
	}
	C1 = FMath::Sqrt(C1);

	if (C1 < Tol)
	{
		return TVec4<TVector<FSolverReal, 3>>(TVector<FSolverReal, 3>((FSolverReal)0.));
	}

	const PMatrix<FSolverReal, 3, 3> DmInvT = ElementDmInv(ElementIndex).GetTransposed();

	const TVec4<TVector<FSolverReal, 3>> DeltaC1 = GetPolarGradient(Fe, Re, DmInvT, C1);

	FSolverReal AlphaTilde = (FSolverReal)1. / (Dt * Dt * MuElementArray[ElementIndex] * Measure[ElementIndex]);

	if (MuElementArray[ElementIndex] > (FSolverReal)1. / (FSolverReal)UE_SMALL_NUMBER)
	{
		AlphaTilde = (FSolverReal)0.;
	}

	if (bRecordMetric)
	{
		HError += C1 + AlphaTilde * LambdaArray[2 * ElementIndex + 0];
		for (int32 TetLocalIdx = 0; TetLocalIdx < 4; TetLocalIdx++)
		{
			for (int32 Alpha = 0; Alpha < 3; Alpha++)
			{
				GError[MeshConstraints[ElementIndex][TetLocalIdx] * 3 + Alpha] -= DeltaC1[TetLocalIdx][Alpha] * LambdaArray[2 * ElementIndex + 0];
			}
		}
		HErrorArray[2 * ElementIndex + 0] = C1 + AlphaTilde * LambdaArray[2 * ElementIndex + 0];
	}

	FSolverReal DLambda = -C1 - AlphaTilde * LambdaArray[2 * ElementIndex + 0];

	FSolverReal Denom = AlphaTilde;
	for (int32 Index = 0; Index < 4; Index++)
	{
		for (int32 Alpha = 0; Alpha < 3; Alpha++)
		{
			Denom += DeltaC1[Index][Alpha] * Particles.InvM(MeshConstraints[ElementIndex][Index]) * DeltaC1[Index][Alpha];
		}
	}
	DLambda /= Denom;
	LambdaArray[2 * ElementIndex + 0] += DLambda;
	TVec4<TVector<FSolverReal, 3>> Delta(TVector<FSolverReal, 3>((FSolverReal)0.));
	for (int32 Index = 0; Index < 4; Index++)
	{
		for (int32 Alpha = 0; Alpha < 3; Alpha++)
		{
			Delta[Index][Alpha] = Particles.InvM(MeshConstraints[ElementIndex][Index]) * DeltaC1[Index][Alpha] * DLambda;
		}
	}
	return Delta;
}

}  // End namespace Chaos::Softs
