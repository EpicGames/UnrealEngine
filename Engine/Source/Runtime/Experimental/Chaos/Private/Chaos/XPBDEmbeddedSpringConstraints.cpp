// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/XPBDEmbeddedSpringConstraints.h"
#include "Chaos/Framework/Parallel.h"
#include "ChaosStats.h"
#include "ChaosLog.h"
#include "HAL/IConsoleManager.h"

#include "XPBDInternal.h"

#if INTEL_ISPC
#include "XPBDEmbeddedSpringConstraints.ispc.generated.h"
#endif

#if INTEL_ISPC && !UE_BUILD_SHIPPING
static_assert(sizeof(ispc::FVector4f) == sizeof(Chaos::Softs::FPAndInvM), "sizeof(ispc::FVector4f) != sizeof(Chaos::Softs::FPAndInvM)");
static_assert(sizeof(ispc::FVector3f) == sizeof(Chaos::Softs::FSolverVec3), "sizeof(ispc::FVector3f) == sizeof(Chaos::Softs::FSolverVec3)");
static_assert(sizeof(ispc::FIntVector2) == sizeof(Chaos::TVec2<int32>), "sizeof(ispc::FIntVector2) != sizeof(Chaos::TVec2<int32>)");
static_assert(sizeof(ispc::FIntVector4) == sizeof(Chaos::TVec4<int32>), "sizeof(ispc::FIntVector4) != sizeof(Chaos::TVec4<int32>)");
static_assert(sizeof(ispc::FIntVector6) == sizeof(Chaos::TVector<int32,6>), "sizeof(ispc::FIntVector4) != sizeof(Chaos::TVec6<int32>)");
static_assert(sizeof(ispc::FVector2f) == sizeof(Chaos::TVec2<Chaos::Softs::FSolverReal>), "sizeof(ispc::FVector2f) != sizeof(Chaos::TVec2<Chaos::Softs::FSolverReal>)");
static_assert(sizeof(ispc::FVector4f) == sizeof(Chaos::TVec4<Chaos::Softs::FSolverReal>), "sizeof(ispc::FVector4f) != sizeof(Chaos::TVec4<Chaos::Softs::FSolverReal>)");
static_assert(sizeof(ispc::FVector6f) == sizeof(Chaos::TVector<Chaos::Softs::FSolverReal,6>), "sizeof(ispc::FVector6f) != sizeof(Chaos::TVec6<Chaos::Softs::FSolverReal>)");
#endif

namespace Chaos::Softs
{
	// @todo(chaos): the parallel threshold (or decision to run parallel) should probably be owned by the solver and passed to the constraint container
	extern int32 Chaos_XPBDSpring_ParallelConstraintCount;

#if INTEL_ISPC
	template<int32 Source, int32 Target>
	void TXPBDEmbeddedSpringConstraints<Source, Target>::InitColor(const FSolverParticlesRange& Particles)
	{
		const int32 NumConstraints = Constraints.Num();
		// In dev builds we always color so we can tune the system without restarting. See Apply()
#if UE_BUILD_SHIPPING || UE_BUILD_TEST
		if (NumConstraints > Chaos_XPBDSpring_ParallelConstraintCount)
#endif
		{
			const TArray<TArray<int32>> ConstraintsPerColor = FGraphColoring::ComputeGraphColoringParticlesOrRange(Constraints, Particles, 0, Particles.Size());

			// Reorder constraints based on color so each array in ConstraintsPerColor contains contiguous elements.
			TArray<TVector<int32, N>> ReorderedConstraints;
			TArray<TVector<FSolverReal, N>> ReorderedWeights;
			TArray<FSolverReal> ReorderedSpringLengths;
			TArray<int32> OrigToReorderedIndices; // used to reorder stiffness indices
			ReorderedConstraints.SetNumUninitialized(NumConstraints);
			ReorderedWeights.SetNumUninitialized(NumConstraints);
			ReorderedSpringLengths.SetNumUninitialized(NumConstraints);
			OrigToReorderedIndices.SetNumUninitialized(NumConstraints);
			ConstraintsPerColorStartIndex.Reset(ConstraintsPerColor.Num() + 1);
			int32 ReorderedIndex = 0;
			for (const TArray<int32>& ConstraintsBatch : ConstraintsPerColor)
			{
				ConstraintsPerColorStartIndex.Add(ReorderedIndex);
				for (const int32 OrigIndex : ConstraintsBatch)
				{
					ReorderedConstraints[ReorderedIndex] = Constraints[OrigIndex];
					ReorderedWeights[ReorderedIndex] = Weights[OrigIndex];
					ReorderedSpringLengths[ReorderedIndex] = SpringLengths[OrigIndex];
					OrigToReorderedIndices[OrigIndex] = ReorderedIndex;

					++ReorderedIndex;
				}
			}
			ConstraintsPerColorStartIndex.Add(ReorderedIndex);

			// Update OrigMapToReordered based on this reordering.
			for (int32 OrigMapIndex = 0; OrigMapIndex < OrigMapToReordered.Num(); ++OrigMapIndex)
			{
				if (OrigMapToReordered[OrigMapIndex] != INDEX_NONE)
				{
					OrigMapToReordered[OrigMapIndex] = OrigToReorderedIndices[OrigMapToReordered[OrigMapIndex]];
				}
			}
			Constraints = MoveTemp(ReorderedConstraints);
			Weights = MoveTemp(ReorderedWeights);
			SpringLengths = MoveTemp(ReorderedSpringLengths);
		}
	}


	template<int32 Source, int32 Target>
	void TXPBDEmbeddedSpringConstraints<Source, Target>::ApplyISPC(FSolverParticlesRange& Particles, const FSolverReal Dt) const
	{
		if (ConstraintsPerColorStartIndex.Num() > 1 && Constraints.Num() > Chaos_XPBDSpring_ParallelConstraintCount)
		{
			const int32 ConstraintColorNum = ConstraintsPerColorStartIndex.Num() - 1;
			const bool bDampingHasWeightMap = DampingRatio.HasWeightMap();
			const FSolverReal DampingNoMap = (FSolverReal)DampingRatio;

			const bool bExtensionStiffnessHasWeightMap = ExtensionStiffness.HasWeightMap();
			const bool bCompressionStiffnessHasWeightMap = CompressionStiffness.HasWeightMap();

			if constexpr (Source == 1 && Target == 1)
			{
				if (DampingNoMap > 0 || bDampingHasWeightMap)
				{
					for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
					{
						const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
						const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
						ispc::ApplyXPBDEmbeddedSpringDampingConstraints(
							(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
							(const ispc::FVector3f*)Particles.XArray().GetData(),
							(ispc::FIntVector2*)&Constraints.GetData()[ColorStart],
							&SpringLengths.GetData()[ColorStart],
							&LambdasDamping.GetData()[ColorStart],
							Dt,
							bExtensionStiffnessHasWeightMap,
							reinterpret_cast<const ispc::FVector2f&>(ExtensionStiffness.GetOffsetRange()),
							bExtensionStiffnessHasWeightMap ? &ExtensionStiffness.GetMapValues().GetData()[ColorStart] : nullptr,
							bCompressionStiffnessHasWeightMap,
							reinterpret_cast<const ispc::FVector2f&>(CompressionStiffness.GetOffsetRange()),
							bCompressionStiffnessHasWeightMap ? &CompressionStiffness.GetMapValues().GetData()[ColorStart] : nullptr,
							bDampingHasWeightMap,
							reinterpret_cast<const ispc::FVector2f&>(DampingRatio.GetOffsetRange()),
							bDampingHasWeightMap ? &DampingRatio.GetMapValues().GetData()[ColorStart] : nullptr,
							ColorSize
						);
					}
				}

				for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
				{
					const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
					const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
					ispc::ApplyXPBDEmbeddedSpringConstraints(
						(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
						(ispc::FIntVector2*)&Constraints.GetData()[ColorStart],
						&SpringLengths.GetData()[ColorStart],
						&Lambdas.GetData()[ColorStart],
						Dt,
						bExtensionStiffnessHasWeightMap,
						reinterpret_cast<const ispc::FVector2f&>(ExtensionStiffness.GetOffsetRange()),
						bExtensionStiffnessHasWeightMap ? &ExtensionStiffness.GetMapValues().GetData()[ColorStart] : nullptr,
						bCompressionStiffnessHasWeightMap,
						reinterpret_cast<const ispc::FVector2f&>(CompressionStiffness.GetOffsetRange()),
						bCompressionStiffnessHasWeightMap ? &CompressionStiffness.GetMapValues().GetData()[ColorStart] : nullptr,
						ColorSize
					);
				}
			}
			else if constexpr (Source == 1 && Target == 3)
			{
				if (DampingNoMap > 0 || bDampingHasWeightMap)
				{
					for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
					{
						const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
						const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
						ispc::ApplyXPBDEmbeddedVertexFaceSpringDampingConstraints(
							(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
							(const ispc::FVector3f*)Particles.XArray().GetData(),
							(ispc::FIntVector4*)&Constraints.GetData()[ColorStart],
							(ispc::FVector4f*)&Weights.GetData()[ColorStart],
							&SpringLengths.GetData()[ColorStart],
							&LambdasDamping.GetData()[ColorStart],
							Dt,
							bExtensionStiffnessHasWeightMap,
							reinterpret_cast<const ispc::FVector2f&>(ExtensionStiffness.GetOffsetRange()),
							bExtensionStiffnessHasWeightMap ? &ExtensionStiffness.GetMapValues().GetData()[ColorStart] : nullptr,
							bCompressionStiffnessHasWeightMap,
							reinterpret_cast<const ispc::FVector2f&>(CompressionStiffness.GetOffsetRange()),
							bCompressionStiffnessHasWeightMap ? &CompressionStiffness.GetMapValues().GetData()[ColorStart] : nullptr,
							bDampingHasWeightMap,
							reinterpret_cast<const ispc::FVector2f&>(DampingRatio.GetOffsetRange()),
							bDampingHasWeightMap ? &DampingRatio.GetMapValues().GetData()[ColorStart] : nullptr,
							ColorSize
						);
					}
				}

				for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
				{
					const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
					const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
					ispc::ApplyXPBDEmbeddedVertexFaceSpringConstraints(
						(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
						(ispc::FIntVector4*)&Constraints.GetData()[ColorStart],
						(ispc::FVector4f*)&Weights.GetData()[ColorStart],
						&SpringLengths.GetData()[ColorStart],
						&Lambdas.GetData()[ColorStart],
						Dt,
						bExtensionStiffnessHasWeightMap,
						reinterpret_cast<const ispc::FVector2f&>(ExtensionStiffness.GetOffsetRange()),
						bExtensionStiffnessHasWeightMap ? &ExtensionStiffness.GetMapValues().GetData()[ColorStart] : nullptr,
						bCompressionStiffnessHasWeightMap,
						reinterpret_cast<const ispc::FVector2f&>(CompressionStiffness.GetOffsetRange()),
						bCompressionStiffnessHasWeightMap ? &CompressionStiffness.GetMapValues().GetData()[ColorStart] : nullptr,
						ColorSize
					);
				}
			}
			else if constexpr (Source == 3 && Target == 3)
			{
				if (DampingNoMap > 0 || bDampingHasWeightMap)
				{
					for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
					{
						const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
						const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
						ispc::ApplyXPBDEmbeddedFaceSpringDampingConstraints(
							(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
							(const ispc::FVector3f*)Particles.XArray().GetData(),
							(ispc::FIntVector6*)&Constraints.GetData()[ColorStart],
							(ispc::FVector6f*)&Weights.GetData()[ColorStart],
							&SpringLengths.GetData()[ColorStart],
							&LambdasDamping.GetData()[ColorStart],
							Dt,
							bExtensionStiffnessHasWeightMap,
							reinterpret_cast<const ispc::FVector2f&>(ExtensionStiffness.GetOffsetRange()),
							bExtensionStiffnessHasWeightMap ? &ExtensionStiffness.GetMapValues().GetData()[ColorStart] : nullptr,
							bCompressionStiffnessHasWeightMap,
							reinterpret_cast<const ispc::FVector2f&>(CompressionStiffness.GetOffsetRange()),
							bCompressionStiffnessHasWeightMap ? &CompressionStiffness.GetMapValues().GetData()[ColorStart] : nullptr,
							bDampingHasWeightMap,
							reinterpret_cast<const ispc::FVector2f&>(DampingRatio.GetOffsetRange()),
							bDampingHasWeightMap ? &DampingRatio.GetMapValues().GetData()[ColorStart] : nullptr,
							ColorSize
						);
					}
				}

				for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
				{
					const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
					const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
					ispc::ApplyXPBDEmbeddedFaceSpringConstraints(
						(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
						(ispc::FIntVector6*)&Constraints.GetData()[ColorStart],
						(ispc::FVector6f*)&Weights.GetData()[ColorStart],
						&SpringLengths.GetData()[ColorStart],
						&Lambdas.GetData()[ColorStart],
						Dt,
						bExtensionStiffnessHasWeightMap,
						reinterpret_cast<const ispc::FVector2f&>(ExtensionStiffness.GetOffsetRange()),
						bExtensionStiffnessHasWeightMap ? &ExtensionStiffness.GetMapValues().GetData()[ColorStart] : nullptr,
						bCompressionStiffnessHasWeightMap,
						reinterpret_cast<const ispc::FVector2f&>(CompressionStiffness.GetOffsetRange()),
						bCompressionStiffnessHasWeightMap ? &CompressionStiffness.GetMapValues().GetData()[ColorStart] : nullptr,
						ColorSize
					);
				}
			}
			else
			{
				checkNoEntry();
				ApplyInternal(Particles, Dt);
			}
		}
		else
		{
			ApplyInternal(Particles, Dt);
		}
	}
#endif
	template class TXPBDEmbeddedSpringConstraints<1, 1>;
	template class TXPBDEmbeddedSpringConstraints<1, 3>;
	template class TXPBDEmbeddedSpringConstraints<3, 3>;

	FXPBDVertexConstraints::FXPBDVertexConstraints(
		const FSolverParticlesRange& Particles,
		const FCollectionPropertyConstFacade& PropertyCollection,
		const FEmbeddedSpringConstraintFacade& SpringConstraintFacade)
		: TXPBDEmbeddedSpringConstraints<1, 1>(
			Particles,
			SpringConstraintFacade.GetSourceIndexConst(),
			SpringConstraintFacade.GetSourceWeightsConst(),
			SpringConstraintFacade.GetTargetIndexConst(),
			SpringConstraintFacade.GetTargetWeightsConst(),
			SpringConstraintFacade.GetSpringLengthConst(),
			SpringConstraintFacade.GetExtensionStiffnessConst(),
			SpringConstraintFacade.GetCompressionStiffnessConst(),
			SpringConstraintFacade.GetDampingConst(),
			FSolverVec2(GetWeightedFloatVertexSpringExtensionStiffness(PropertyCollection, DefaultStiffness)),
			FSolverVec2(GetWeightedFloatVertexSpringCompressionStiffness(PropertyCollection, DefaultStiffness)),
			FSolverVec2(GetWeightedFloatVertexSpringDamping(PropertyCollection, DefaultDamping))
		)
		, VertexSpringExtensionStiffnessIndex(PropertyCollection)
		, VertexSpringCompressionStiffnessIndex(PropertyCollection)
		, VertexSpringDampingIndex(PropertyCollection)
	{
	}

	void FXPBDVertexConstraints::SetProperties(const FCollectionPropertyConstFacade& PropertyCollection)
	{
		if (IsVertexSpringExtensionStiffnessMutable(PropertyCollection))
		{
			const FSolverVec2 WeightedValue(GetWeightedFloatVertexSpringExtensionStiffness(PropertyCollection));
			ExtensionStiffness.SetWeightedValue(WeightedValue.ClampAxes(MinStiffness, MaxStiffness));
		}
		if (IsVertexSpringCompressionStiffnessMutable(PropertyCollection))
		{
			const FSolverVec2 WeightedValue(GetWeightedFloatVertexSpringCompressionStiffness(PropertyCollection));
			CompressionStiffness.SetWeightedValue(WeightedValue.ClampAxes(MinStiffness, MaxStiffness));
		}
		if (IsVertexSpringDampingMutable(PropertyCollection))
		{
			const FSolverVec2 WeightedValue(GetWeightedFloatVertexSpringDamping(PropertyCollection));
			DampingRatio.SetWeightedValue(WeightedValue.ClampAxes(MinDampingRatio, MaxDampingRatio));
		}
	}

	FXPBDVertexFaceConstraints::FXPBDVertexFaceConstraints(
		const FSolverParticlesRange& Particles,
		const FCollectionPropertyConstFacade& PropertyCollection,
		const FEmbeddedSpringConstraintFacade& SpringConstraintFacade)
		: TXPBDEmbeddedSpringConstraints<1, 3>(
			Particles,
			SpringConstraintFacade.GetSourceIndexConst(),
			SpringConstraintFacade.GetSourceWeightsConst(),
			SpringConstraintFacade.GetTargetIndexConst(),
			SpringConstraintFacade.GetTargetWeightsConst(),
			SpringConstraintFacade.GetSpringLengthConst(),
			SpringConstraintFacade.GetExtensionStiffnessConst(),
			SpringConstraintFacade.GetCompressionStiffnessConst(),
			SpringConstraintFacade.GetDampingConst(),
			FSolverVec2(GetWeightedFloatVertexFaceSpringExtensionStiffness(PropertyCollection, DefaultStiffness)),
			FSolverVec2(GetWeightedFloatVertexFaceSpringCompressionStiffness(PropertyCollection, DefaultStiffness)),
			FSolverVec2(GetWeightedFloatVertexFaceSpringDamping(PropertyCollection, DefaultDamping))
		)
		, VertexFaceSpringExtensionStiffnessIndex(PropertyCollection)
		, VertexFaceSpringCompressionStiffnessIndex(PropertyCollection)
		, VertexFaceSpringDampingIndex(PropertyCollection)
	{
	}

	void FXPBDVertexFaceConstraints::SetProperties(const FCollectionPropertyConstFacade& PropertyCollection)
	{
		if (IsVertexFaceSpringExtensionStiffnessMutable(PropertyCollection))
		{
			const FSolverVec2 WeightedValue(GetWeightedFloatVertexFaceSpringExtensionStiffness(PropertyCollection));
			ExtensionStiffness.SetWeightedValue(WeightedValue.ClampAxes(MinStiffness, MaxStiffness));
		}
		if (IsVertexFaceSpringCompressionStiffnessMutable(PropertyCollection))
		{
			const FSolverVec2 WeightedValue(GetWeightedFloatVertexFaceSpringCompressionStiffness(PropertyCollection));
			CompressionStiffness.SetWeightedValue(WeightedValue.ClampAxes(MinStiffness, MaxStiffness));
		}
		if (IsVertexFaceSpringDampingMutable(PropertyCollection))
		{
			const FSolverVec2 WeightedValue(GetWeightedFloatVertexFaceSpringDamping(PropertyCollection));
			DampingRatio.SetWeightedValue(WeightedValue.ClampAxes(MinDampingRatio, MaxDampingRatio));
		}
	}

	FXPBDFaceConstraints::FXPBDFaceConstraints(
		const FSolverParticlesRange& Particles,
		const FCollectionPropertyConstFacade& PropertyCollection,
		const FEmbeddedSpringConstraintFacade& SpringConstraintFacade)
		: TXPBDEmbeddedSpringConstraints<3, 3>(
			Particles,
			SpringConstraintFacade.GetSourceIndexConst(),
			SpringConstraintFacade.GetSourceWeightsConst(),
			SpringConstraintFacade.GetTargetIndexConst(),
			SpringConstraintFacade.GetTargetWeightsConst(),
			SpringConstraintFacade.GetSpringLengthConst(),
			SpringConstraintFacade.GetExtensionStiffnessConst(),
			SpringConstraintFacade.GetCompressionStiffnessConst(),
			SpringConstraintFacade.GetDampingConst(),
			FSolverVec2(GetWeightedFloatFaceSpringExtensionStiffness(PropertyCollection, DefaultStiffness)),
			FSolverVec2(GetWeightedFloatFaceSpringCompressionStiffness(PropertyCollection, DefaultStiffness)),
			FSolverVec2(GetWeightedFloatFaceSpringDamping(PropertyCollection, DefaultDamping))
		)
		, FaceSpringExtensionStiffnessIndex(PropertyCollection)
		, FaceSpringCompressionStiffnessIndex(PropertyCollection)
		, FaceSpringDampingIndex(PropertyCollection)
	{
	}

	void FXPBDFaceConstraints::SetProperties(const FCollectionPropertyConstFacade& PropertyCollection)
	{
		if (IsFaceSpringExtensionStiffnessMutable(PropertyCollection))
		{
			const FSolverVec2 WeightedValue(GetWeightedFloatFaceSpringExtensionStiffness(PropertyCollection));
			ExtensionStiffness.SetWeightedValue(WeightedValue.ClampAxes(MinStiffness, MaxStiffness));
		}
		if (IsFaceSpringCompressionStiffnessMutable(PropertyCollection))
		{
			const FSolverVec2 WeightedValue(GetWeightedFloatFaceSpringCompressionStiffness(PropertyCollection));
			CompressionStiffness.SetWeightedValue(WeightedValue.ClampAxes(MinStiffness, MaxStiffness));
		}
		if (IsFaceSpringDampingMutable(PropertyCollection))
		{
			const FSolverVec2 WeightedValue(GetWeightedFloatFaceSpringDamping(PropertyCollection));
			DampingRatio.SetWeightedValue(WeightedValue.ClampAxes(MinDampingRatio, MaxDampingRatio));
		}
	}
} // End namespace Chaos::Softs
