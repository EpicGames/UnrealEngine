// Copyright Epic Games, Inc. All Rights Reserved.

#define EXPLICIT_VECTOR4 1

#include "Math/Vector.isph"
#include "Chaos/PBDSofts.isph"
#include "Chaos/SoftsSpring.isph"

struct FIntVector6
{
	int V[6];
};

struct FVector6f
{
	float V[6];
};

inline varying FIntVector6 VectorLoad(const uniform FIntVector6 *uniform SrcPtr)
{
	varying FIntVector6 Result;

	if(((1<<TARGET_WIDTH)-1 ^ lanemask()) == 0)
	{
		aos_to_soa6_ispc((uniform int32 * uniform)SrcPtr, &Result.V[0], &Result.V[1], &Result.V[2], &Result.V[3], &Result.V[4], &Result.V[5]);
	}
	else
	{
		#pragma ignore warning(perf)
		Result = SrcPtr[programIndex];
	}

	return Result;
}

inline varying FVector6f VectorLoad(const uniform FVector6f *uniform SrcPtr)
{
	varying FVector6f Result;

	if(((1<<TARGET_WIDTH)-1 ^ lanemask()) == 0)
	{
		aos_to_soa6_ispc((uniform float * uniform)SrcPtr, &Result.V[0], &Result.V[1], &Result.V[2], &Result.V[3], &Result.V[4], &Result.V[5]);
	}
	else
	{
		#pragma ignore warning(perf)
		Result = SrcPtr[programIndex];
	}

	return Result;
}

export void ApplyXPBDEmbeddedSpringConstraints(uniform FVector4f PandInvM[],
									const uniform FIntVector2 Constraints[],
									const uniform float Dists[],
									uniform float Lambdas[],
									const uniform float Dt,
									const uniform bool bExtensionStiffnessHasMap,
									const uniform FVector2f& ExtensionStiffnessOffsetRange,
									const uniform float ExtensionStiffnessMapValues[],
									const uniform bool bCompressionStiffnessHasMap,
									const uniform FVector2f& CompressionStiffnessOffsetRange,
									const uniform float CompressionStiffnessMapValues[],
									const uniform int32 NumConstraints)
{

	foreach(ConstraintIndex = 0 ... NumConstraints)
	{
		const varying FIntVector2 Constraint = VectorLoad(&Constraints[extract(ConstraintIndex,0)]);
		const varying int32 Index1 = Constraint.V[0];
		const varying int32 Index2 = Constraint.V[1];

		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[Index1]);
		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[Index2]);

		varying FVector3f P1, P2;
		varying float InvM1, InvM2;
		UnzipPandInvM(PandInvM1, P1, InvM1);
		UnzipPandInvM(PandInvM2, P2, InvM2);

		const varying float Dist = Dists[ConstraintIndex];
		const varying float Lambda = Lambdas[ConstraintIndex];
		
		const varying float ExtensionStiffness = bExtensionStiffnessHasMap ? ExtensionStiffnessOffsetRange.V[0] + ExtensionStiffnessMapValues[ConstraintIndex]*ExtensionStiffnessOffsetRange.V[1] : ExtensionStiffnessOffsetRange.V[0];
		const varying float CompressionStiffness = bCompressionStiffnessHasMap ? CompressionStiffnessOffsetRange.V[0] + CompressionStiffnessMapValues[ConstraintIndex]*CompressionStiffnessOffsetRange.V[1] : CompressionStiffnessOffsetRange.V[0];

		varying float DLambda;
		const varying FVector3f Delta = GetXPBDEmbeddedSpringDelta(P1, P2, InvM1, InvM2, Dt, Dist, Lambda, ExtensionStiffness, CompressionStiffness, DLambda);

		Lambdas[ConstraintIndex] = Lambda + DLambda;
		
		VectorScatter(&PandInvM[Index1], SetVector4( P1 + (InvM1 * Delta), InvM1 ));
		VectorScatter(&PandInvM[Index2], SetVector4( P2 - (InvM2 * Delta), InvM2 ));
	}
}

export void ApplyXPBDEmbeddedSpringDampingConstraints(uniform FVector4f PandInvM[],
													const uniform FVector3f X[],
													const uniform FIntVector2 Constraints[],
													const uniform float Dists[],
													uniform float Lambdas[],
													const uniform float Dt,
													const uniform bool bExtensionStiffnessHasMap,
													const uniform FVector2f& ExtensionStiffnessOffsetRange,
													const uniform float ExtensionStiffnessMapValues[],
													const uniform bool bCompressionStiffnessHasMap,
													const uniform FVector2f& CompressionStiffnessOffsetRange,
													const uniform float CompressionStiffnessMapValues[],
													const uniform bool bDampingHasMap,
													const uniform FVector2f& DampingOffsetRange,
													const uniform float DampingMapValues[],
													const uniform int32 NumConstraints)
{

	foreach(ConstraintIndex = 0 ... NumConstraints)
	{
		const varying FIntVector2 Constraint = VectorLoad(&Constraints[extract(ConstraintIndex,0)]);
		const varying int32 Index1 = Constraint.V[0];
		const varying int32 Index2 = Constraint.V[1];

		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[Index1]);
		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[Index2]);

		varying FVector3f P1, P2;
		varying float InvM1, InvM2;
		UnzipPandInvM(PandInvM1, P1, InvM1);
		UnzipPandInvM(PandInvM2, P2, InvM2);

		const varying FVector3f X1 = VectorGather(&X[Index1]);
		const varying FVector3f X2 = VectorGather(&X[Index2]);
		
		const varying float Dist = Dists[ConstraintIndex];
		const varying float Lambda = Lambdas[ConstraintIndex];
		
		const varying float ExtensionStiffness = bExtensionStiffnessHasMap ? ExtensionStiffnessOffsetRange.V[0] + ExtensionStiffnessMapValues[ConstraintIndex]*ExtensionStiffnessOffsetRange.V[1] : ExtensionStiffnessOffsetRange.V[0];
		const varying float CompressionStiffness = bCompressionStiffnessHasMap ? CompressionStiffnessOffsetRange.V[0] + CompressionStiffnessMapValues[ConstraintIndex]*CompressionStiffnessOffsetRange.V[1] : CompressionStiffnessOffsetRange.V[0];
		const varying float DampingRatio = bDampingHasMap ? DampingOffsetRange.V[0] + DampingMapValues[ConstraintIndex]*DampingOffsetRange.V[1] : DampingOffsetRange.V[0];

		varying float DLambda;
		const FVector3f Delta = GetXPBDEmbeddedSpringDampingDelta(P1, P2, X1, X2, InvM1, InvM2, Dt, Dist, Lambda, ExtensionStiffness, CompressionStiffness, DampingRatio, DLambda);
		
		Lambdas[ConstraintIndex] = Lambda + DLambda;
		
		VectorScatter(&PandInvM[Index1], SetVector4( P1 + (InvM1 * Delta), InvM1 ));
		VectorScatter(&PandInvM[Index2], SetVector4( P2 - (InvM2 * Delta), InvM2 ));
	}
}

export void ApplyXPBDEmbeddedVertexFaceSpringConstraints(uniform FVector4f PandInvM[],
									const uniform FIntVector4 Constraints[],
									const uniform FVector4f Weights[],
									const uniform float Dists[],
									uniform float Lambdas[],
									const uniform float Dt,
									const uniform bool bExtensionStiffnessHasMap,
									const uniform FVector2f& ExtensionStiffnessOffsetRange,
									const uniform float ExtensionStiffnessMapValues[],
									const uniform bool bCompressionStiffnessHasMap,
									const uniform FVector2f& CompressionStiffnessOffsetRange,
									const uniform float CompressionStiffnessMapValues[],
									const uniform int32 NumConstraints)
{

	foreach(ConstraintIndex = 0 ... NumConstraints)
	{
		const varying FIntVector4 Constraint = VectorLoad(&Constraints[extract(ConstraintIndex,0)]);
		const varying int32 Index1 = Constraint.V[0];
		const varying int32 Index2 = Constraint.V[1];
		const varying int32 Index3 = Constraint.V[2];
		const varying int32 Index4 = Constraint.V[3];
		const varying FVector4f Weight = VectorLoad(&Weights[extract(ConstraintIndex,0)]);

		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[Index1]);
		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[Index2]);
		const varying FVector4f PandInvM3 = VectorGather(&PandInvM[Index3]);
		const varying FVector4f PandInvM4 = VectorGather(&PandInvM[Index4]);

		varying FVector3f P1, P2, P3, P4;
		varying float InvM1, InvM2, InvM3, InvM4;
		UnzipPandInvM(PandInvM1, P1, InvM1);
		UnzipPandInvM(PandInvM2, P2, InvM2);
		UnzipPandInvM(PandInvM3, P3, InvM3);
		UnzipPandInvM(PandInvM4, P4, InvM4);
		
		const varying FVector3f P = -Weight.V[1] * P2 - Weight.V[2] * P3 - Weight.V[3] * P4;
		const varying float InvM = -Weight.V[1] * InvM2 - Weight.V[2] * InvM3 - Weight.V[3] * InvM4;

		const varying float Dist = Dists[ConstraintIndex];
		const varying float Lambda = Lambdas[ConstraintIndex];
		
		const varying float ExtensionStiffness = bExtensionStiffnessHasMap ? ExtensionStiffnessOffsetRange.V[0] + ExtensionStiffnessMapValues[ConstraintIndex]*ExtensionStiffnessOffsetRange.V[1] : ExtensionStiffnessOffsetRange.V[0];
		const varying float CompressionStiffness = bCompressionStiffnessHasMap ? CompressionStiffnessOffsetRange.V[0] + CompressionStiffnessMapValues[ConstraintIndex]*CompressionStiffnessOffsetRange.V[1] : CompressionStiffnessOffsetRange.V[0];

		varying float DLambda;
		const varying FVector3f Delta = GetXPBDEmbeddedSpringDelta(P1, P, InvM1, InvM, Dt, Dist, Lambda, ExtensionStiffness, CompressionStiffness, DLambda);

		Lambdas[ConstraintIndex] = Lambda + DLambda;
		
		VectorScatter(&PandInvM[Index1], SetVector4( P1 + (InvM1 * Delta), InvM1 ));
		VectorScatter(&PandInvM[Index2], SetVector4( P2 + Weight.V[1] * (InvM2 * Delta), InvM2 ));
		VectorScatter(&PandInvM[Index3], SetVector4( P3 + Weight.V[2] * (InvM3 * Delta), InvM3 ));
		VectorScatter(&PandInvM[Index4], SetVector4( P4 + Weight.V[3] * (InvM4 * Delta), InvM4 ));
	}
}

export void ApplyXPBDEmbeddedVertexFaceSpringDampingConstraints(uniform FVector4f PandInvM[],
													const uniform FVector3f XArray[],
													const uniform FIntVector4 Constraints[],
													const uniform FVector4f Weights[],
													const uniform float Dists[],
													uniform float Lambdas[],
													const uniform float Dt,
													const uniform bool bExtensionStiffnessHasMap,
													const uniform FVector2f& ExtensionStiffnessOffsetRange,
													const uniform float ExtensionStiffnessMapValues[],
													const uniform bool bCompressionStiffnessHasMap,
													const uniform FVector2f& CompressionStiffnessOffsetRange,
													const uniform float CompressionStiffnessMapValues[],
													const uniform bool bDampingHasMap,
													const uniform FVector2f& DampingOffsetRange,
													const uniform float DampingMapValues[],
													const uniform int32 NumConstraints)
{

	foreach(ConstraintIndex = 0 ... NumConstraints)
	{
		const varying FIntVector4 Constraint = VectorLoad(&Constraints[extract(ConstraintIndex,0)]);
		const varying int32 Index1 = Constraint.V[0];
		const varying int32 Index2 = Constraint.V[1];
		const varying int32 Index3 = Constraint.V[2];
		const varying int32 Index4 = Constraint.V[3];
		const varying FVector4f Weight = VectorLoad(&Weights[extract(ConstraintIndex,0)]);

		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[Index1]);
		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[Index2]);
		const varying FVector4f PandInvM3 = VectorGather(&PandInvM[Index3]);
		const varying FVector4f PandInvM4 = VectorGather(&PandInvM[Index4]);

		varying FVector3f P1, P2, P3, P4;
		varying float InvM1, InvM2, InvM3, InvM4;
		UnzipPandInvM(PandInvM1, P1, InvM1);
		UnzipPandInvM(PandInvM2, P2, InvM2);
		UnzipPandInvM(PandInvM3, P3, InvM3);
		UnzipPandInvM(PandInvM4, P4, InvM4);

		const varying FVector3f P = -Weight.V[1] * P2 - Weight.V[2] * P3 - Weight.V[3] * P4;
		const varying float InvM = -Weight.V[1] * InvM2 - Weight.V[2] * InvM3 - Weight.V[3] * InvM4;

		const varying FVector3f X1 = VectorGather(&XArray[Index1]);
		const varying FVector3f X2 = VectorGather(&XArray[Index2]);
		const varying FVector3f X3 = VectorGather(&XArray[Index3]);
		const varying FVector3f X4 = VectorGather(&XArray[Index4]);
		
		const varying FVector3f X = -Weight.V[1] * X2 - Weight.V[2] * X3 - Weight.V[3] * X4;
		
		const varying float Dist = Dists[ConstraintIndex];
		const varying float Lambda = Lambdas[ConstraintIndex];
		
		const varying float ExtensionStiffness = bExtensionStiffnessHasMap ? ExtensionStiffnessOffsetRange.V[0] + ExtensionStiffnessMapValues[ConstraintIndex]*ExtensionStiffnessOffsetRange.V[1] : ExtensionStiffnessOffsetRange.V[0];
		const varying float CompressionStiffness = bCompressionStiffnessHasMap ? CompressionStiffnessOffsetRange.V[0] + CompressionStiffnessMapValues[ConstraintIndex]*CompressionStiffnessOffsetRange.V[1] : CompressionStiffnessOffsetRange.V[0];
		const varying float DampingRatio = bDampingHasMap ? DampingOffsetRange.V[0] + DampingMapValues[ConstraintIndex]*DampingOffsetRange.V[1] : DampingOffsetRange.V[0];

		varying float DLambda;
		const FVector3f Delta = GetXPBDEmbeddedSpringDampingDelta(P1, P, X1, X, InvM1, InvM, Dt, Dist, Lambda, ExtensionStiffness, CompressionStiffness, DampingRatio, DLambda);
		
		Lambdas[ConstraintIndex] = Lambda + DLambda;
		
		VectorScatter(&PandInvM[Index1], SetVector4( P1 + (InvM1 * Delta), InvM1 ));
		VectorScatter(&PandInvM[Index2], SetVector4( P2 + Weight.V[1] * (InvM2 * Delta), InvM2 ));
		VectorScatter(&PandInvM[Index3], SetVector4( P3 + Weight.V[2] * (InvM3 * Delta), InvM3 ));
		VectorScatter(&PandInvM[Index4], SetVector4( P4 + Weight.V[3] * (InvM4 * Delta), InvM4 ));
	}
}

export void ApplyXPBDEmbeddedFaceSpringConstraints(uniform FVector4f PandInvM[],
									const uniform FIntVector6 Constraints[],
									const uniform FVector6f Weights[],
									const uniform float Dists[],
									uniform float Lambdas[],
									const uniform float Dt,
									const uniform bool bExtensionStiffnessHasMap,
									const uniform FVector2f& ExtensionStiffnessOffsetRange,
									const uniform float ExtensionStiffnessMapValues[],
									const uniform bool bCompressionStiffnessHasMap,
									const uniform FVector2f& CompressionStiffnessOffsetRange,
									const uniform float CompressionStiffnessMapValues[],
									const uniform int32 NumConstraints)
{

	foreach(ConstraintIndex = 0 ... NumConstraints)
	{
		const varying FIntVector6 Constraint = VectorLoad(&Constraints[extract(ConstraintIndex,0)]);
		const varying int32 Index1 = Constraint.V[0];
		const varying int32 Index2 = Constraint.V[1];
		const varying int32 Index3 = Constraint.V[2];
		const varying int32 Index4 = Constraint.V[3];
		const varying int32 Index5 = Constraint.V[4];
		const varying int32 Index6 = Constraint.V[5];
		const varying FVector6f Weight = VectorLoad(&Weights[extract(ConstraintIndex,0)]);

		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[Index1]);
		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[Index2]);
		const varying FVector4f PandInvM3 = VectorGather(&PandInvM[Index3]);
		const varying FVector4f PandInvM4 = VectorGather(&PandInvM[Index4]);
		const varying FVector4f PandInvM5 = VectorGather(&PandInvM[Index5]);
		const varying FVector4f PandInvM6 = VectorGather(&PandInvM[Index6]);

		varying FVector3f P1, P2, P3, P4, P5, P6;
		varying float InvM1, InvM2, InvM3, InvM4, InvM5, InvM6;
		UnzipPandInvM(PandInvM1, P1, InvM1);
		UnzipPandInvM(PandInvM2, P2, InvM2);
		UnzipPandInvM(PandInvM3, P3, InvM3);
		UnzipPandInvM(PandInvM4, P4, InvM4);
		UnzipPandInvM(PandInvM5, P5, InvM5);
		UnzipPandInvM(PandInvM6, P6, InvM6);
		
		const varying FVector3f Ps =  Weight.V[0] * P1 + Weight.V[1] * P2 + Weight.V[2] * P3;
		const varying FVector3f Pt = -Weight.V[3] * P4 - Weight.V[4] * P5 - Weight.V[5] * P6;
		const varying float InvMs =  Weight.V[0] * InvM1 + Weight.V[1] * InvM2 + Weight.V[2] * InvM3;
		const varying float InvMt = -Weight.V[3] * InvM4 - Weight.V[4] * InvM5 - Weight.V[5] * InvM6;

		const varying float Dist = Dists[ConstraintIndex];
		const varying float Lambda = Lambdas[ConstraintIndex];
		
		const varying float ExtensionStiffness = bExtensionStiffnessHasMap ? ExtensionStiffnessOffsetRange.V[0] + ExtensionStiffnessMapValues[ConstraintIndex]*ExtensionStiffnessOffsetRange.V[1] : ExtensionStiffnessOffsetRange.V[0];
		const varying float CompressionStiffness = bCompressionStiffnessHasMap ? CompressionStiffnessOffsetRange.V[0] + CompressionStiffnessMapValues[ConstraintIndex]*CompressionStiffnessOffsetRange.V[1] : CompressionStiffnessOffsetRange.V[0];

		varying float DLambda;
		const varying FVector3f Delta = GetXPBDEmbeddedSpringDelta(Ps, Pt, InvMs, InvMt, Dt, Dist, Lambda, ExtensionStiffness, CompressionStiffness, DLambda);

		Lambdas[ConstraintIndex] = Lambda + DLambda;
		
		VectorScatter(&PandInvM[Index1], SetVector4( P1 + Weight.V[0] * (InvM1 * Delta), InvM1 ));
		VectorScatter(&PandInvM[Index2], SetVector4( P2 + Weight.V[1] * (InvM2 * Delta), InvM2 ));
		VectorScatter(&PandInvM[Index3], SetVector4( P3 + Weight.V[2] * (InvM3 * Delta), InvM3 ));
		VectorScatter(&PandInvM[Index4], SetVector4( P4 + Weight.V[3] * (InvM4 * Delta), InvM4 ));
		VectorScatter(&PandInvM[Index5], SetVector4( P5 + Weight.V[4] * (InvM5 * Delta), InvM5 ));
		VectorScatter(&PandInvM[Index6], SetVector4( P6 + Weight.V[5] * (InvM6 * Delta), InvM6 ));
	}
}

export void ApplyXPBDEmbeddedFaceSpringDampingConstraints(uniform FVector4f PandInvM[],
													const uniform FVector3f XArray[],
													const uniform FIntVector6 Constraints[],
													const uniform FVector6f Weights[],
													const uniform float Dists[],
													uniform float Lambdas[],
													const uniform float Dt,
													const uniform bool bExtensionStiffnessHasMap,
													const uniform FVector2f& ExtensionStiffnessOffsetRange,
													const uniform float ExtensionStiffnessMapValues[],
													const uniform bool bCompressionStiffnessHasMap,
													const uniform FVector2f& CompressionStiffnessOffsetRange,
													const uniform float CompressionStiffnessMapValues[],
													const uniform bool bDampingHasMap,
													const uniform FVector2f& DampingOffsetRange,
													const uniform float DampingMapValues[],
													const uniform int32 NumConstraints)
{

	foreach(ConstraintIndex = 0 ... NumConstraints)
	{
		const varying FIntVector6 Constraint = VectorLoad(&Constraints[extract(ConstraintIndex,0)]);
		const varying int32 Index1 = Constraint.V[0];
		const varying int32 Index2 = Constraint.V[1];
		const varying int32 Index3 = Constraint.V[2];
		const varying int32 Index4 = Constraint.V[3];
		const varying int32 Index5 = Constraint.V[4];
		const varying int32 Index6 = Constraint.V[5];
		const varying FVector6f Weight = VectorLoad(&Weights[extract(ConstraintIndex,0)]);

		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[Index1]);
		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[Index2]);
		const varying FVector4f PandInvM3 = VectorGather(&PandInvM[Index3]);
		const varying FVector4f PandInvM4 = VectorGather(&PandInvM[Index4]);
		const varying FVector4f PandInvM5 = VectorGather(&PandInvM[Index5]);
		const varying FVector4f PandInvM6 = VectorGather(&PandInvM[Index6]);

		varying FVector3f P1, P2, P3, P4, P5, P6;
		varying float InvM1, InvM2, InvM3, InvM4, InvM5, InvM6;
		UnzipPandInvM(PandInvM1, P1, InvM1);
		UnzipPandInvM(PandInvM2, P2, InvM2);
		UnzipPandInvM(PandInvM3, P3, InvM3);
		UnzipPandInvM(PandInvM4, P4, InvM4);
		UnzipPandInvM(PandInvM5, P5, InvM5);
		UnzipPandInvM(PandInvM6, P6, InvM6);
		
		const varying FVector3f Ps =  Weight.V[0] * P1 + Weight.V[1] * P2 + Weight.V[2] * P3;
		const varying FVector3f Pt = -Weight.V[3] * P4 - Weight.V[4] * P5 - Weight.V[5] * P6;
		const varying float InvMs =  Weight.V[0] * InvM1 + Weight.V[1] * InvM2 + Weight.V[2] * InvM3;
		const varying float InvMt = -Weight.V[3] * InvM4 - Weight.V[4] * InvM5 - Weight.V[5] * InvM6;

		const varying FVector3f X1 = VectorGather(&XArray[Index1]);
		const varying FVector3f X2 = VectorGather(&XArray[Index2]);
		const varying FVector3f X3 = VectorGather(&XArray[Index3]);
		const varying FVector3f X4 = VectorGather(&XArray[Index4]);
		const varying FVector3f X5 = VectorGather(&XArray[Index5]);
		const varying FVector3f X6 = VectorGather(&XArray[Index6]);
		
		const varying FVector3f Xs =  Weight.V[0] * X1 + Weight.V[1] * X2 + Weight.V[2] * X3;
		const varying FVector3f Xt = -Weight.V[3] * X4 - Weight.V[4] * X5 - Weight.V[5] * X6;
		
		const varying float Dist = Dists[ConstraintIndex];
		const varying float Lambda = Lambdas[ConstraintIndex];
		
		const varying float ExtensionStiffness = bExtensionStiffnessHasMap ? ExtensionStiffnessOffsetRange.V[0] + ExtensionStiffnessMapValues[ConstraintIndex]*ExtensionStiffnessOffsetRange.V[1] : ExtensionStiffnessOffsetRange.V[0];
		const varying float CompressionStiffness = bCompressionStiffnessHasMap ? CompressionStiffnessOffsetRange.V[0] + CompressionStiffnessMapValues[ConstraintIndex]*CompressionStiffnessOffsetRange.V[1] : CompressionStiffnessOffsetRange.V[0];
		const varying float DampingRatio = bDampingHasMap ? DampingOffsetRange.V[0] + DampingMapValues[ConstraintIndex]*DampingOffsetRange.V[1] : DampingOffsetRange.V[0];

		varying float DLambda;
		const FVector3f Delta = GetXPBDEmbeddedSpringDampingDelta(Ps, Pt, Xs, Xt, InvMs, InvMt, Dt, Dist, Lambda, ExtensionStiffness, CompressionStiffness, DampingRatio, DLambda);
		
		Lambdas[ConstraintIndex] = Lambda + DLambda;
		
		VectorScatter(&PandInvM[Index1], SetVector4( P1 + Weight.V[0] * (InvM1 * Delta), InvM1 ));
		VectorScatter(&PandInvM[Index2], SetVector4( P2 + Weight.V[1] * (InvM2 * Delta), InvM2 ));
		VectorScatter(&PandInvM[Index3], SetVector4( P3 + Weight.V[2] * (InvM3 * Delta), InvM3 ));
		VectorScatter(&PandInvM[Index4], SetVector4( P4 + Weight.V[3] * (InvM4 * Delta), InvM4 ));
		VectorScatter(&PandInvM[Index5], SetVector4( P5 + Weight.V[4] * (InvM5 * Delta), InvM5 ));
		VectorScatter(&PandInvM[Index6], SetVector4( P6 + Weight.V[5] * (InvM6 * Delta), InvM6 ));
	}
}
