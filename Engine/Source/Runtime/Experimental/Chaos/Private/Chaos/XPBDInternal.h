// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HAL/Platform.h"

namespace Chaos::Softs
{
// Used for controlling damping model for xpbd springs (CVar-based testing only)
enum struct EXPBDSplitDampingMode
{
	SingleLambda = 0,
	InterleavedAfter = 1,
	InterleavedBefore = 2,
	TwoPassAfter = 3,
	TwoPassBefore = 4
};
#if UE_BUILD_SHIPPING
const int32 Chaos_XPBDSpring_SplitDampingMode = (int32)EXPBDSplitDampingMode::TwoPassBefore;
#else
extern int32 Chaos_XPBDSpring_SplitDampingMode;
#endif

}  // End namespace Chaos::Softs
