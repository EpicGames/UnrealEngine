// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/XPBDSpringConstraints.h"
#include "Chaos/Framework/Parallel.h"
#include "Chaos/GraphColoring.h"
#include "Chaos/PBDSoftsSolverParticles.h"
#include "Chaos/SoftsEvolutionLinearSystem.h"
#include "ChaosStats.h"
#include "ChaosLog.h"
#include "HAL/IConsoleManager.h"

#include "XPBDInternal.h"

#if INTEL_ISPC
#include "XPBDSpringConstraints.ispc.generated.h"
#endif

DECLARE_CYCLE_STAT(TEXT("Chaos XPBD Spring Constraint"), STAT_XPBD_Spring, STATGROUP_Chaos);

#if INTEL_ISPC && !UE_BUILD_SHIPPING
static_assert(sizeof(ispc::FVector4f) == sizeof(Chaos::Softs::FPAndInvM), "sizeof(ispc::FVector4f) != sizeof(Chaos::Softs::FPAndInvM)");
static_assert(sizeof(ispc::FVector3f) == sizeof(Chaos::Softs::FSolverVec3), "sizeof(ispc::FVector3f) == sizeof(Chaos::Softs::FSolverVec3)");
static_assert(sizeof(ispc::FIntVector2) == sizeof(Chaos::TVec2<int32>), "sizeof(ispc::FIntVector2) != sizeof(Chaos::TVec2<int32>)");

bool bChaos_XPBDSpring_ISPC_Enabled = CHAOS_XPBD_SPRING_ISPC_ENABLED_DEFAULT;
FAutoConsoleVariableRef CVarChaosXPBDSpringISPCEnabled(TEXT("p.Chaos.XPBDSpring.ISPC"), bChaos_XPBDSpring_ISPC_Enabled, TEXT("Whether to use ISPC optimizations in XPBD Spring constraints"));
#endif

namespace Chaos::Softs {

// @todo(chaos): the parallel threshold (or decision to run parallel) should probably be owned by the solver and passed to the constraint container
int32 Chaos_XPBDSpring_ParallelConstraintCount = 100;
#if !UE_BUILD_SHIPPING
FAutoConsoleVariableRef CVarChaosXPBDSpringParallelConstraintCount(TEXT("p.Chaos.XPBDSpring.ParallelConstraintCount"), Chaos_XPBDSpring_ParallelConstraintCount, TEXT("If we have more constraints than this, use parallel-for in Apply."));

int32 Chaos_XPBDSpring_SplitDampingMode = (int32)EXPBDSplitDampingMode::TwoPassBefore;
FAutoConsoleVariableRef CVarChaosXPBDSpringSplitDamping(TEXT("p.Chaos.XPBDSpring.SplitDamping"), Chaos_XPBDSpring_SplitDampingMode, TEXT("Test xpbd spring split damping mode. 0 = single lambda, 1 = interleaved with damping after (non-ispc only), 2 interleaved with damping before (non-ispc only), 3 =  two passes damping after (non-ispc only), 4 = two passes damping before (default)."));
#endif

template<typename SolverParticlesOrRange>
void FXPBDSpringConstraints::InitColor(const SolverParticlesOrRange& Particles)
{
	// In dev builds we always color so we can tune the system without restarting. See Apply()
#if UE_BUILD_SHIPPING || UE_BUILD_TEST
	if (Constraints.Num() > Chaos_XPBDSpring_ParallelConstraintCount)
#endif
	{
		const TArray<TArray<int32>> ConstraintsPerColor = FGraphColoring::ComputeGraphColoringParticlesOrRange(Constraints, Particles, ParticleOffset, ParticleOffset + ParticleCount);
		
		// Reorder constraints based on color so each array in ConstraintsPerColor contains contiguous elements.
		TArray<TVec2<int32>> ReorderedConstraints;
		TArray<FSolverReal> ReorderedDists;
		TArray<int32> OrigToReorderedIndices; // used to reorder stiffness indices
		ReorderedConstraints.SetNumUninitialized(Constraints.Num());
		ReorderedDists.SetNumUninitialized(Dists.Num());
		OrigToReorderedIndices.SetNumUninitialized(Constraints.Num());

		ConstraintsPerColorStartIndex.Reset(ConstraintsPerColor.Num() + 1);

		int32 ReorderedIndex = 0;
		for (const TArray<int32>& ConstraintsBatch : ConstraintsPerColor)
		{
			ConstraintsPerColorStartIndex.Add(ReorderedIndex);
			for (const int32& BatchConstraint : ConstraintsBatch)
			{
				const int32 OrigIndex = BatchConstraint;
				ReorderedConstraints[ReorderedIndex] = Constraints[OrigIndex];
				ReorderedDists[ReorderedIndex] = Dists[OrigIndex];
				OrigToReorderedIndices[OrigIndex] = ReorderedIndex;

				++ReorderedIndex;
			}
		}
		ConstraintsPerColorStartIndex.Add(ReorderedIndex);

		Constraints = MoveTemp(ReorderedConstraints);
		Dists = MoveTemp(ReorderedDists);
		Stiffness.ReorderIndices(OrigToReorderedIndices);
		DampingRatio.ReorderIndices(OrigToReorderedIndices);
	}
}
template CHAOS_API void FXPBDSpringConstraints::InitColor(const FSolverParticles& Particles);
template CHAOS_API void FXPBDSpringConstraints::InitColor(const FSolverParticlesRange& Particles);

template<bool bDampingBefore, bool bSingleLambda, bool bSeparateStretch, bool bDampingAfter, typename SolverParticlesOrRange>
void FXPBDSpringConstraints::ApplyHelper(SolverParticlesOrRange& Particles, const FSolverReal Dt, const int32 ConstraintIndex, const FSolverReal ExpStiffnessValue, const FSolverReal DampingRatioValue) const
{
	const TVec2<int32>& Constraint = Constraints[ConstraintIndex];
	const int32 Index1 = Constraint[0];
	const int32 Index2 = Constraint[1];

	FSolverVec3 Delta(0.f);
	if constexpr (bDampingBefore)
	{
		Delta += Spring::GetXPBDSpringDampingDelta(Particles, Dt, Constraint, Dists[ConstraintIndex], LambdasDamping[ConstraintIndex],
			ExpStiffnessValue, DampingRatioValue);
	}

	if constexpr (bSingleLambda)
	{
		Delta += Spring::GetXPBDSpringDeltaWithDamping(Particles, Dt, Constraint, Dists[ConstraintIndex], Lambdas[ConstraintIndex],
			ExpStiffnessValue, DampingRatioValue);
	}

	if constexpr (bSeparateStretch)
	{
		Delta += Spring::GetXPBDSpringDelta(Particles, Dt, Constraint, Dists[ConstraintIndex], Lambdas[ConstraintIndex],
			ExpStiffnessValue);
	}

	if constexpr (bDampingAfter)
	{
		Delta += Spring::GetXPBDSpringDampingDelta(Particles, Dt, Constraint, Dists[ConstraintIndex], LambdasDamping[ConstraintIndex],
			ExpStiffnessValue, DampingRatioValue);
	}
	
	Particles.P(Index1) += Particles.InvM(Index1) * Delta;
	Particles.P(Index2) -= Particles.InvM(Index2) * Delta;
}

template<typename SolverParticlesOrRange>
void FXPBDSpringConstraints::Apply(SolverParticlesOrRange& Particles, const FSolverReal Dt) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FXPBDSpringConstraints_Apply);
	SCOPE_CYCLE_COUNTER(STAT_XPBD_Spring);

	const bool StiffnessHasWeightMap = Stiffness.HasWeightMap();
	const bool DampingHasWeightMap = DampingRatio.HasWeightMap();

	if ((ConstraintsPerColorStartIndex.Num() > 1) && (Constraints.Num() > Chaos_XPBDSpring_ParallelConstraintCount))
	{
		const int32 ConstraintColorNum = ConstraintsPerColorStartIndex.Num() - 1;
#if INTEL_ISPC
		if (bRealTypeCompatibleWithISPC && bChaos_XPBDSpring_ISPC_Enabled)
		{
			const bool bSingleLambda = Chaos_XPBDSpring_SplitDampingMode == (int32)EXPBDSplitDampingMode::SingleLambda;
			if (!StiffnessHasWeightMap && !DampingHasWeightMap)
			{
				const FSolverReal ExpStiffnessValue = (FSolverReal)Stiffness;
				const FSolverReal DampingRatioValue = (FSolverReal)DampingRatio;
				if (ExpStiffnessValue < MinStiffness)
				{
					return;
				}
				
				if (DampingRatioValue > 0)
				{
					if (bSingleLambda)
					{
						for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
						{
							const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
							const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
							ispc::ApplyXPBDSpringConstraintsWithDamping(
								(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
								(const ispc::FVector3f*)Particles.XArray().GetData(),
								(ispc::FIntVector2*)&Constraints.GetData()[ColorStart],
								&Dists.GetData()[ColorStart],
								&Lambdas.GetData()[ColorStart],
								Dt,
								ExpStiffnessValue,
								DampingRatioValue,
								ColorSize);
						}
						return;
					}
					else
					{
						for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
						{
							const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
							const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
							ispc::ApplyXPBDSpringDampingConstraints(
								(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
								(const ispc::FVector3f*)Particles.XArray().GetData(),
								(ispc::FIntVector2*)&Constraints.GetData()[ColorStart],
								&Dists.GetData()[ColorStart],
								&LambdasDamping.GetData()[ColorStart],
								Dt,
								ExpStiffnessValue,
								DampingRatioValue,
								ColorSize);
						}
					}
				}

				for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
				{
					const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
					const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
					ispc::ApplyXPBDSpringConstraints(
						(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
						(ispc::FIntVector2*)&Constraints.GetData()[ColorStart],
						&Dists.GetData()[ColorStart],
						&Lambdas.GetData()[ColorStart],
						Dt,
						ExpStiffnessValue,
						ColorSize);
				}
			}
			else // ISPC with maps
			{
				if (DampingHasWeightMap || (FSolverReal)DampingRatio > 0)
				{
					if (bSingleLambda)
					{
						for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
						{
							const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
							const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
							ispc::ApplyXPBDSpringConstraintsWithDampingAndWeightMaps(
								(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
								(const ispc::FVector3f*)Particles.XArray().GetData(),
								(ispc::FIntVector2*)&Constraints.GetData()[ColorStart],
								&Dists.GetData()[ColorStart],
								&Lambdas.GetData()[ColorStart],
								Dt,
								StiffnessHasWeightMap,
								StiffnessHasWeightMap ? &Stiffness.GetIndices().GetData()[ColorStart] : nullptr,
								&Stiffness.GetTable().GetData()[0],
								DampingHasWeightMap,
								DampingHasWeightMap ? &DampingRatio.GetIndices().GetData()[ColorStart] : nullptr,
								&DampingRatio.GetTable().GetData()[0],
								ColorSize);
						}
						return;
					}
					else
					{
						for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
						{
							const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
							const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
							ispc::ApplyXPBDSpringDampingConstraintsWithWeightMaps(
								(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
								(const ispc::FVector3f*)Particles.XArray().GetData(),
								(ispc::FIntVector2*)&Constraints.GetData()[ColorStart],
								&Dists.GetData()[ColorStart],
								&LambdasDamping.GetData()[ColorStart],
								Dt,
								StiffnessHasWeightMap,
								StiffnessHasWeightMap ? &Stiffness.GetIndices().GetData()[ColorStart] : nullptr,
								&Stiffness.GetTable().GetData()[0],
								DampingHasWeightMap,
								DampingHasWeightMap ? &DampingRatio.GetIndices().GetData()[ColorStart] : nullptr,
								&DampingRatio.GetTable().GetData()[0],
								ColorSize);
						}
					}
				}
				for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
				{
					const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
					const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
					ispc::ApplyXPBDSpringConstraintsWithWeightMaps(
						(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
						(ispc::FIntVector2*)&Constraints.GetData()[ColorStart],
						&Dists.GetData()[ColorStart],
						&Lambdas.GetData()[ColorStart],
						Dt,
						&Stiffness.GetIndices().GetData()[ColorStart],
						&Stiffness.GetTable().GetData()[0],
						ColorSize);
				}
			}
		}
		else
#endif
		{
			// Parallel non-ispc
			const FSolverReal StiffnessNoMap = (FSolverReal)Stiffness;
			const FSolverReal DampingNoMap = (FSolverReal)DampingRatio;
			if (DampingNoMap > 0 || DampingHasWeightMap)
			{
				if (Chaos_XPBDSpring_SplitDampingMode == (int32)EXPBDSplitDampingMode::TwoPassBefore)
				{
					for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
					{
						const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
						const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
						PhysicsParallelFor(ColorSize, [this, &Particles, Dt, ColorStart, StiffnessHasWeightMap, StiffnessNoMap, DampingHasWeightMap, DampingNoMap](const int32 Index)
						{
							const int32 ConstraintIndex = ColorStart + Index;
							constexpr bool bDampingBefore = true;
							constexpr bool bSingleLambda = false;
							constexpr bool bSeparateStretch = false;
							constexpr bool bDampingAfter = false;
							const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? Stiffness[ConstraintIndex] : StiffnessNoMap;
							const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
							ApplyHelper<bDampingBefore, bSingleLambda, bSeparateStretch, bDampingAfter>(Particles, Dt, ConstraintIndex, ExpStiffnessValue, DampingRatioValue);
						});
					}
				}

				// Stretch part (possibly with damping depending on split mode)
				switch (Chaos_XPBDSpring_SplitDampingMode)
				{
				case (int32)EXPBDSplitDampingMode::TwoPassBefore: // fallthrough
				case (int32)EXPBDSplitDampingMode::TwoPassAfter: // fallthrough
				default:
				{
					// Do a pass with stretch only
					for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
					{
						const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
						const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
						PhysicsParallelFor(ColorSize, [this, &Particles, Dt, ColorStart, StiffnessHasWeightMap, StiffnessNoMap, DampingHasWeightMap, DampingNoMap](const int32 Index)
						{
							const int32 ConstraintIndex = ColorStart + Index;
							constexpr bool bDampingBefore = false;
							constexpr bool bSingleLambda = false;
							constexpr bool bSeparateStretch = true;
							constexpr bool bDampingAfter = false;
							const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? Stiffness[ConstraintIndex] : StiffnessNoMap;
							const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
							ApplyHelper<bDampingBefore, bSingleLambda, bSeparateStretch, bDampingAfter>(Particles, Dt, ConstraintIndex, ExpStiffnessValue, DampingRatioValue);
						});
					}
				}
				break;
				case (int32)EXPBDSplitDampingMode::SingleLambda:
				{
					// Do a pass with combined stretch and damping with a single lambda
					for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
					{
						const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
						const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
						PhysicsParallelFor(ColorSize, [this, &Particles, Dt, ColorStart, StiffnessHasWeightMap, StiffnessNoMap, DampingHasWeightMap, DampingNoMap](const int32 Index)
						{
							const int32 ConstraintIndex = ColorStart + Index;
							constexpr bool bDampingBefore = false;
							constexpr bool bSingleLambda = true;
							constexpr bool bSeparateStretch = false;
							constexpr bool bDampingAfter = false;
							const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? Stiffness[ConstraintIndex] : StiffnessNoMap;
							const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
							ApplyHelper<bDampingBefore, bSingleLambda, bSeparateStretch, bDampingAfter>(Particles, Dt, ConstraintIndex, ExpStiffnessValue, DampingRatioValue);
						});
					}
				}
				break;
				case (int32)EXPBDSplitDampingMode::InterleavedBefore:
				{
					// Do a pass with damping before stretch
					for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
					{
						const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
						const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
						PhysicsParallelFor(ColorSize, [this, &Particles, Dt, ColorStart, StiffnessHasWeightMap, StiffnessNoMap, DampingHasWeightMap, DampingNoMap](const int32 Index)
						{
							const int32 ConstraintIndex = ColorStart + Index;
							constexpr bool bDampingBefore = true;
							constexpr bool bSingleLambda = false;
							constexpr bool bSeparateStretch = true;
							constexpr bool bDampingAfter = false;
							const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? Stiffness[ConstraintIndex] : StiffnessNoMap;
							const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
							ApplyHelper<bDampingBefore, bSingleLambda, bSeparateStretch, bDampingAfter>(Particles, Dt, ConstraintIndex, ExpStiffnessValue, DampingRatioValue);
						});
					}
				}
				break;
				case (int32)EXPBDSplitDampingMode::InterleavedAfter:
				{
					// Do a pass with damping after stretch
					for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
					{
						const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
						const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
						PhysicsParallelFor(ColorSize, [this, &Particles, Dt, ColorStart, StiffnessHasWeightMap, StiffnessNoMap, DampingHasWeightMap, DampingNoMap](const int32 Index)
						{
							const int32 ConstraintIndex = ColorStart + Index;
							constexpr bool bDampingBefore = false;
							constexpr bool bSingleLambda = false;
							constexpr bool bSeparateStretch = true;
							constexpr bool bDampingAfter = true;
							const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? Stiffness[ConstraintIndex] : StiffnessNoMap;
							const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
							ApplyHelper<bDampingBefore, bSingleLambda, bSeparateStretch, bDampingAfter>(Particles, Dt, ConstraintIndex, ExpStiffnessValue, DampingRatioValue);
						});
					}
				}
				break;
				}
				if (Chaos_XPBDSpring_SplitDampingMode == (int32)EXPBDSplitDampingMode::TwoPassAfter)
				{
					for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
					{
						const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
						const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
						PhysicsParallelFor(ColorSize, [this, &Particles, Dt, ColorStart, StiffnessHasWeightMap, StiffnessNoMap, DampingHasWeightMap, DampingNoMap](const int32 Index)
						{
							const int32 ConstraintIndex = ColorStart + Index;
							constexpr bool bDampingBefore = false;
							constexpr bool bSingleLambda = false;
							constexpr bool bSeparateStretch = false;
							constexpr bool bDampingAfter = true;
							const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? Stiffness[ConstraintIndex] : StiffnessNoMap;
							const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
							ApplyHelper<bDampingBefore, bSingleLambda, bSeparateStretch, bDampingAfter>(Particles, Dt, ConstraintIndex, ExpStiffnessValue, DampingRatioValue);
						});
					}
				}
			}
			else
			{
				// No damping. 
				for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
				{
					const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
					const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
					PhysicsParallelFor(ColorSize, [this, &Particles, Dt, ColorStart, StiffnessHasWeightMap, StiffnessNoMap](const int32 Index)
					{
						const int32 ConstraintIndex = ColorStart + Index;
						constexpr bool bDampingBefore = false;
						constexpr bool bSingleLambda = false;
						constexpr bool bSeparateStretch = true;
						constexpr bool bDampingAfter = false;
						const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? Stiffness[ConstraintIndex] : StiffnessNoMap;
						ApplyHelper<bDampingBefore, bSingleLambda, bSeparateStretch, bDampingAfter>(Particles, Dt, ConstraintIndex, ExpStiffnessValue, (FSolverReal)0.);
					});
				}
			}
		}
	}
	else
	{
		// Single threaded
		const FSolverReal StiffnessNoMap = (FSolverReal)Stiffness;
		const FSolverReal DampingNoMap = (FSolverReal)DampingRatio;
		if (DampingNoMap > 0 || DampingHasWeightMap)
		{
			if (Chaos_XPBDSpring_SplitDampingMode == (int32)EXPBDSplitDampingMode::TwoPassBefore)
			{
				for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
				{
					constexpr bool bDampingBefore = true;
					constexpr bool bSingleLambda = false;
					constexpr bool bSeparateStretch = false;
					constexpr bool bDampingAfter = false;
					const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? Stiffness[ConstraintIndex] : StiffnessNoMap;
					const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
					ApplyHelper<bDampingBefore, bSingleLambda, bSeparateStretch, bDampingAfter>(Particles, Dt, ConstraintIndex, ExpStiffnessValue, DampingRatioValue);
				}
			}

			// Stretch part (possibly with damping depending on split mode)
			switch (Chaos_XPBDSpring_SplitDampingMode)
			{
			case (int32)EXPBDSplitDampingMode::TwoPassBefore: // fallthrough
			case (int32)EXPBDSplitDampingMode::TwoPassAfter: // fallthrough
			default:
			{
				// Do a pass with stretch only
				for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
				{
					constexpr bool bDampingBefore = false;
					constexpr bool bSingleLambda = false;
					constexpr bool bSeparateStretch = true;
					constexpr bool bDampingAfter = false;
					const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? Stiffness[ConstraintIndex] : StiffnessNoMap;
					const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
					ApplyHelper<bDampingBefore, bSingleLambda, bSeparateStretch, bDampingAfter>(Particles, Dt, ConstraintIndex, ExpStiffnessValue, DampingRatioValue);
				}
			}
			break;
			case (int32)EXPBDSplitDampingMode::SingleLambda:
			{
				// Do a pass with combined stretch and damping with a single lambda
				for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
				{
					constexpr bool bDampingBefore = false;
					constexpr bool bSingleLambda = true;
					constexpr bool bSeparateStretch = false;
					constexpr bool bDampingAfter = false;
					const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? Stiffness[ConstraintIndex] : StiffnessNoMap;
					const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
					ApplyHelper<bDampingBefore, bSingleLambda, bSeparateStretch, bDampingAfter>(Particles, Dt, ConstraintIndex, ExpStiffnessValue, DampingRatioValue);
				}
			}
			break;
			case (int32)EXPBDSplitDampingMode::InterleavedBefore:
			{
				// Do a pass with damping before stretch
				for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
				{
					constexpr bool bDampingBefore = true;
					constexpr bool bSingleLambda = false;
					constexpr bool bSeparateStretch = true;
					constexpr bool bDampingAfter = false;
					const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? Stiffness[ConstraintIndex] : StiffnessNoMap;
					const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
					ApplyHelper<bDampingBefore, bSingleLambda, bSeparateStretch, bDampingAfter>(Particles, Dt, ConstraintIndex, ExpStiffnessValue, DampingRatioValue);
				}
			}
			break;
			case (int32)EXPBDSplitDampingMode::InterleavedAfter:
			{
				// Do a pass with damping after stretch
				for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
				{
					constexpr bool bDampingBefore = false;
					constexpr bool bSingleLambda = false;
					constexpr bool bSeparateStretch = true;
					constexpr bool bDampingAfter = true;
					const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? Stiffness[ConstraintIndex] : StiffnessNoMap;
					const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
					ApplyHelper<bDampingBefore, bSingleLambda, bSeparateStretch, bDampingAfter>(Particles, Dt, ConstraintIndex, ExpStiffnessValue, DampingRatioValue);
				}
			}
			break;
			}
			if (Chaos_XPBDSpring_SplitDampingMode == (int32)EXPBDSplitDampingMode::TwoPassAfter)
			{
				for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
				{
					constexpr bool bDampingBefore = false;
					constexpr bool bSingleLambda = false;
					constexpr bool bSeparateStretch = false;
					constexpr bool bDampingAfter = true;
					const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? Stiffness[ConstraintIndex] : StiffnessNoMap;
					const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
					ApplyHelper<bDampingBefore, bSingleLambda, bSeparateStretch, bDampingAfter>(Particles, Dt, ConstraintIndex, ExpStiffnessValue, DampingRatioValue);
				}
			}
		}
		else
		{
			// No damping. 
			for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
			{
				constexpr bool bDampingBefore = false;
				constexpr bool bSingleLambda = false;
				constexpr bool bSeparateStretch = true;
				constexpr bool bDampingAfter = false;
				const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? Stiffness[ConstraintIndex] : StiffnessNoMap;
				const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
				ApplyHelper<bDampingBefore, bSingleLambda, bSeparateStretch, bDampingAfter>(Particles, Dt, ConstraintIndex, ExpStiffnessValue, (FSolverReal)0.);

			}
		}
	}
}
template CHAOS_API void FXPBDSpringConstraints::Apply(FSolverParticles& Particles, const FSolverReal Dt) const;
template CHAOS_API void FXPBDSpringConstraints::Apply(FSolverParticlesRange& Particles, const FSolverReal Dt) const;

void FXPBDSpringConstraints::UpdateLinearSystem(const FSolverParticlesRange& Particles, const FSolverReal Dt, FEvolutionLinearSystem& LinearSystem) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FXPBDSpringConstraints_UpdateLinearSystem);
	LinearSystem.ReserveForParallelAdd(Constraints.Num() * 2, Constraints.Num());

	const bool StiffnessHasWeightMap = Stiffness.HasWeightMap();
	const bool DampingHasWeightMap = DampingRatio.HasWeightMap();
	if ((ConstraintsPerColorStartIndex.Num() > 1) && (Constraints.Num() > Chaos_XPBDSpring_ParallelConstraintCount))
	{
		const int32 ConstraintColorNum = ConstraintsPerColorStartIndex.Num() - 1;
		if (!StiffnessHasWeightMap && !DampingHasWeightMap)
		{
			const FSolverReal ExpStiffnessValue = (FSolverReal)Stiffness;
			const FSolverReal DampingRatioValue = (FSolverReal)DampingRatio;
			if (ExpStiffnessValue < MinStiffness)
			{
				return;
			}
			for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
			{
				const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
				const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
				PhysicsParallelFor(ColorSize, [this, &Particles, Dt, &LinearSystem, ColorStart, ExpStiffnessValue, DampingRatioValue](const int32 Index)
				{
					const int32 ConstraintIndex = ColorStart + Index;
					Spring::UpdateSpringLinearSystem(Particles, Dt, Constraints[ConstraintIndex], Dists[ConstraintIndex], ExpStiffnessValue, MinStiffness, DampingRatioValue, LinearSystem);
				});
			}
		}
		else // Has weight maps
		{
			const FSolverReal StiffnessNoMap = (FSolverReal)Stiffness;
			const FSolverReal DampingNoMap = (FSolverReal)DampingRatio;
			for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
			{
				const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
				const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
				PhysicsParallelFor(ColorSize, [this, &Particles, Dt, &LinearSystem, ColorStart, StiffnessHasWeightMap, StiffnessNoMap, DampingHasWeightMap, DampingNoMap](const int32 Index)
				{
					const int32 ConstraintIndex = ColorStart + Index;
					const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? Stiffness[ConstraintIndex] : StiffnessNoMap;
					const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
					Spring::UpdateSpringLinearSystem(Particles, Dt, Constraints[ConstraintIndex], Dists[ConstraintIndex], ExpStiffnessValue, MinStiffness, DampingRatioValue, LinearSystem);
				});
			}
		}		
	}
	else
	{
		if (!StiffnessHasWeightMap && !DampingHasWeightMap)
		{
			const FSolverReal ExpStiffnessValue = (FSolverReal)Stiffness;
			const FSolverReal DampingRatioValue = (FSolverReal)DampingRatio;
			if (ExpStiffnessValue < MinStiffness)
			{
				return;
			}
			for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
			{
				Spring::UpdateSpringLinearSystem(Particles, Dt, Constraints[ConstraintIndex], Dists[ConstraintIndex], ExpStiffnessValue, MinStiffness, DampingRatioValue, LinearSystem);
			}
		}
		else
		{
			const FSolverReal StiffnessNoMap = (FSolverReal)Stiffness;
			const FSolverReal DampingNoMap = (FSolverReal)DampingRatio;
			for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
			{
				const FSolverReal ExpStiffnessValue = StiffnessHasWeightMap ? Stiffness[ConstraintIndex] : StiffnessNoMap;
				const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
				Spring::UpdateSpringLinearSystem(Particles, Dt, Constraints[ConstraintIndex], Dists[ConstraintIndex], ExpStiffnessValue, MinStiffness, DampingRatioValue, LinearSystem);
			}
		}
	}
}

void FXPBDEdgeSpringConstraints::SetProperties(
	const FCollectionPropertyConstFacade& PropertyCollection,
	const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps)
{
	if (IsXPBDEdgeSpringStiffnessMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedValue(GetWeightedFloatXPBDEdgeSpringStiffness(PropertyCollection));
		if (IsXPBDEdgeSpringStiffnessStringDirty(PropertyCollection))
		{
			const FString& WeightMapName = GetXPBDEdgeSpringStiffnessString(PropertyCollection);
			Stiffness = FPBDStiffness(
				WeightedValue,
				WeightMaps.FindRef(WeightMapName),
				TConstArrayView<TVec2<int32>>(Constraints),
				ParticleOffset,
				ParticleCount,
				FPBDStiffness::DefaultTableSize,
				FPBDStiffness::DefaultParameterFitBase,
				MaxStiffness);
		}
		else
		{
			Stiffness.SetWeightedValue(WeightedValue, MaxStiffness);
		}
	}
	if (IsXPBDEdgeSpringDampingMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedValue = FSolverVec2(GetWeightedFloatXPBDEdgeSpringDamping(PropertyCollection)).ClampAxes(MinDampingRatio, MaxDampingRatio);
		if (IsXPBDEdgeSpringDampingStringDirty(PropertyCollection))
		{
			const FString& WeightMapName = GetXPBDEdgeSpringDampingString(PropertyCollection);
			DampingRatio = FPBDWeightMap(
				WeightedValue,
				WeightMaps.FindRef(WeightMapName),
				TConstArrayView<TVec2<int32>>(Constraints),
				ParticleOffset,
				ParticleCount);
		}
		else
		{
			DampingRatio.SetWeightedValue(WeightedValue);
		}
	}
}

void FXPBDBendingSpringConstraints::SetProperties(
	const FCollectionPropertyConstFacade& PropertyCollection,
	const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps)
{
	if (IsXPBDBendingSpringStiffnessMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedValue(GetWeightedFloatXPBDBendingSpringStiffness(PropertyCollection));
		if (IsXPBDBendingSpringStiffnessStringDirty(PropertyCollection))
		{
			const FString& WeightMapName = GetXPBDBendingSpringStiffnessString(PropertyCollection);
			Stiffness = FPBDStiffness(
				WeightedValue,
				WeightMaps.FindRef(WeightMapName),
				TConstArrayView<TVec2<int32>>(Constraints),
				ParticleOffset,
				ParticleCount,
				FPBDStiffness::DefaultTableSize,
				FPBDStiffness::DefaultParameterFitBase,
				MaxStiffness);
		}
		else
		{
			Stiffness.SetWeightedValue(WeightedValue, MaxStiffness);
		}
	}
	if (IsXPBDBendingSpringDampingMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedValue = FSolverVec2(GetWeightedFloatXPBDBendingSpringDamping(PropertyCollection)).ClampAxes(MinDampingRatio, MaxDampingRatio);
		if (IsXPBDBendingSpringDampingStringDirty(PropertyCollection))
		{
			const FString& WeightMapName = GetXPBDBendingSpringDampingString(PropertyCollection);
			DampingRatio = FPBDWeightMap(
				WeightedValue,
				WeightMaps.FindRef(WeightMapName),
				TConstArrayView<TVec2<int32>>(Constraints),
				ParticleOffset,
				ParticleCount);
		}
		else
		{
			DampingRatio.SetWeightedValue(WeightedValue);
		}
	}
}

} // End namespace Chaos::Softs
