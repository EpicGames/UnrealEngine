// Copyright Epic Games, Inc. All Rights Reserved.

#define EXPLICIT_VECTOR4 1

#include "Math/Vector.isph"
#include "Chaos/PBDSofts.isph"
#include "Chaos/SoftsSpring.isph"

export void ApplyXPBDSpringConstraints(uniform FVector4f PandInvM[],
									const uniform FIntVector2 Constraints[],
									const uniform float Dists[],
									uniform float Lambdas[],
									const uniform float Dt,
									const uniform float Stiffness,
									const uniform int32 NumConstraints)
{
	foreach(i = 0 ... NumConstraints)
	{
		const varying FIntVector2 Constraint = VectorLoad(&Constraints[extract(i,0)]);
		const varying int32 i1 = Constraint.V[0];
		const varying int32 i2 = Constraint.V[1];

		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[i1]);
		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[i2]);

		varying FVector3f P1, P2;
		varying float InvM1, InvM2;
		UnzipPandInvM(PandInvM1, P1, InvM1);
		UnzipPandInvM(PandInvM2, P2, InvM2);
		
		const varying float Dist = Dists[i];
		const varying float Lambda = Lambdas[i];

		varying float DLambda;
		const varying FVector3f Delta = GetXPBDSpringDelta(P1, P2, InvM1, InvM2, Dt, Dist, Lambda, Stiffness, DLambda);

		Lambdas[i] = Lambda + DLambda;
		VectorScatter(&PandInvM[i1], SetVector4( P1 + (InvM1 * Delta), InvM1 ));
		VectorScatter(&PandInvM[i2], SetVector4( P2 - (InvM2 * Delta), InvM2 ));
	}
}

export void ApplyXPBDSpringConstraintsWithDamping(uniform FVector4f PandInvM[],
												const uniform FVector3f X[],
												const uniform FIntVector2 Constraints[],
												const uniform float Dists[],
												uniform float Lambdas[],
												const uniform float Dt,
												const uniform float Stiffness,
												const uniform float DampingRatio,
												const uniform int32 NumConstraints)
{
	foreach(i = 0 ... NumConstraints)
	{
		const varying FIntVector2 Constraint = VectorLoad(&Constraints[extract(i,0)]);
		const varying int32 i1 = Constraint.V[0];
		const varying int32 i2 = Constraint.V[1];

		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[i1]);
		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[i2]);

		varying FVector3f P1, P2;
		varying float InvM1, InvM2;
		UnzipPandInvM(PandInvM1, P1, InvM1);
		UnzipPandInvM(PandInvM2, P2, InvM2);

		const varying FVector3f X1 = VectorGather(&X[i1]);
		const varying FVector3f X2 = VectorGather(&X[i2]);

		const varying float Dist = Dists[i];
		const varying float Lambda = Lambdas[i];

		varying float DLambda;
		const FVector3f Delta = GetXPBDSpringDeltaWithDamping(P1, P2, X1, X2, InvM1, InvM2, Dt, Dist, Lambda, Stiffness, DampingRatio, DLambda);

		Lambdas[i] = Lambda + DLambda;
		VectorScatter(&PandInvM[i1], SetVector4( P1 + (InvM1 * Delta), InvM1 ));
		VectorScatter(&PandInvM[i2], SetVector4( P2 - (InvM2 * Delta), InvM2 ));
	}
}

export void ApplyXPBDSpringDampingConstraints(uniform FVector4f PandInvM[],
												const uniform FVector3f X[],
												const uniform FIntVector2 Constraints[],
												const uniform float Dists[],
												uniform float Lambdas[],
												const uniform float Dt,
												const uniform float Stiffness,
												const uniform float DampingRatio,
												const uniform int32 NumConstraints)
{
	foreach(i = 0 ... NumConstraints)
	{
		const varying FIntVector2 Constraint = VectorLoad(&Constraints[extract(i,0)]);
		const varying int32 i1 = Constraint.V[0];
		const varying int32 i2 = Constraint.V[1];

		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[i1]);
		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[i2]);

		varying FVector3f P1, P2;
		varying float InvM1, InvM2;
		UnzipPandInvM(PandInvM1, P1, InvM1);
		UnzipPandInvM(PandInvM2, P2, InvM2);

		const varying FVector3f X1 = VectorGather(&X[i1]);
		const varying FVector3f X2 = VectorGather(&X[i2]);

		const varying float Dist = Dists[i];
		const varying float Lambda = Lambdas[i];

		varying float DLambda;
		const FVector3f Delta = GetXPBDSpringDampingDelta(P1, P2, X1, X2, InvM1, InvM2, Dt, Dist, Lambda, Stiffness, DampingRatio, DLambda);

		Lambdas[i] = Lambda + DLambda;
		
		VectorScatter(&PandInvM[i1], SetVector4( P1 + (InvM1 * Delta), InvM1 ));
		VectorScatter(&PandInvM[i2], SetVector4( P2 - (InvM2 * Delta), InvM2 ));
	}
}

export void ApplyXPBDSpringConstraintsWithWeightMaps(uniform FVector4f PandInvM[],
									const uniform FIntVector2 Constraints[],
									const uniform float Dists[],
									uniform float Lambdas[],
									const uniform float Dt,
									const uniform uint8 StiffnessIndices[],
									const uniform float StiffnessTable[],
									const uniform int32 NumConstraints)
{

	foreach(i = 0 ... NumConstraints)
	{
		const varying FIntVector2 Constraint = VectorLoad(&Constraints[extract(i,0)]);
		const varying int32 i1 = Constraint.V[0];
		const varying int32 i2 = Constraint.V[1];

		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[i1]);
		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[i2]);

		varying FVector3f P1, P2;
		varying float InvM1, InvM2;
		UnzipPandInvM(PandInvM1, P1, InvM1);
		UnzipPandInvM(PandInvM2, P2, InvM2);

		const varying float Dist = Dists[i];
		const varying float Lambda = Lambdas[i];

		const varying uint8 StiffnessIndex = StiffnessIndices[i];

		#pragma ignore warning(perf)
		const varying float Stiffness = StiffnessTable[StiffnessIndex];

		varying float DLambda;
		const varying FVector3f Delta = GetXPBDSpringDelta(P1, P2, InvM1, InvM2, Dt, Dist, Lambda, Stiffness, DLambda);

		Lambdas[i] = Lambda + DLambda;
		
		VectorScatter(&PandInvM[i1], SetVector4( P1 + (InvM1 * Delta), InvM1 ));
		VectorScatter(&PandInvM[i2], SetVector4( P2 + (InvM2 * Delta), InvM2 ));
	}
}

export void ApplyXPBDSpringConstraintsWithDampingAndWeightMaps(uniform FVector4f PandInvM[],
															const uniform FVector3f X[],
															const uniform FIntVector2 Constraints[],
															const uniform float Dists[],
															uniform float Lambdas[],
															const uniform float Dt,
															const uniform bool StiffnessHasMap,
															const uniform uint8 StiffnessIndices[],
															const uniform float StiffnessTable[],
															const uniform bool DampingHasMap,
															const uniform uint8 DampingIndices[],
															const uniform float DampingTable[],
															const uniform int32 NumConstraints)
{

	foreach(i = 0 ... NumConstraints)
	{
		const varying FIntVector2 Constraint = VectorLoad(&Constraints[extract(i,0)]);
		const varying int32 i1 = Constraint.V[0];
		const varying int32 i2 = Constraint.V[1];
		const varying int8 StiffnessIndex = StiffnessHasMap ? StiffnessIndices[i] : 0;
		const varying int8 DampingIndex = DampingHasMap ? DampingIndices[i] : 0;

		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[i1]);
		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[i2]);

		varying FVector3f P1, P2;
		varying float InvM1, InvM2;
		UnzipPandInvM(PandInvM1, P1, InvM1);
		UnzipPandInvM(PandInvM2, P2, InvM2);

		const varying FVector3f X1 = VectorGather(&X[i1]);
		const varying FVector3f X2 = VectorGather(&X[i2]);

		const varying float Dist = Dists[i];
		const varying float Lambda = Lambdas[i];

		#pragma ignore warning(perf)
		const varying float Stiffness = StiffnessTable[StiffnessIndex];
		#pragma ignore warning(perf)
		const varying float DampingRatio = DampingTable[DampingIndex];

		varying float DLambda;
		const FVector3f Delta = GetXPBDSpringDeltaWithDamping(P1, P2, X1, X2, InvM1, InvM2, Dt, Dist, Lambda, Stiffness, DampingRatio, DLambda);

		Lambdas[i] = Lambda + DLambda;
		
		VectorScatter(&PandInvM[i1], SetVector4( P1 + (InvM1 * Delta), InvM1 ));
		VectorScatter(&PandInvM[i2], SetVector4( P2 - (InvM2 * Delta), InvM2 ));
	}
}

export void ApplyXPBDSpringDampingConstraintsWithWeightMaps(uniform FVector4f PandInvM[],
															const uniform FVector3f X[],
															const uniform FIntVector2 Constraints[],
															const uniform float Dists[],
															uniform float Lambdas[],
															const uniform float Dt,
															const uniform bool StiffnessHasMap,
															const uniform uint8 StiffnessIndices[],
															const uniform float StiffnessTable[],
															const uniform bool DampingHasMap,
															const uniform uint8 DampingIndices[],
															const uniform float DampingTable[],
															const uniform int32 NumConstraints)
{

	foreach(i = 0 ... NumConstraints)
	{
		const varying FIntVector2 Constraint = VectorLoad(&Constraints[extract(i,0)]);
		const varying int32 i1 = Constraint.V[0];
		const varying int32 i2 = Constraint.V[1];
		const varying int8 StiffnessIndex = StiffnessHasMap ? StiffnessIndices[i] : 0;
		const varying int8 DampingIndex = DampingHasMap ? DampingIndices[i] : 0;

		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[i1]);
		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[i2]);

		varying FVector3f P1, P2;
		varying float InvM1, InvM2;
		UnzipPandInvM(PandInvM1, P1, InvM1);
		UnzipPandInvM(PandInvM2, P2, InvM2);

		const varying FVector3f X1 = VectorGather(&X[i1]);
		const varying FVector3f X2 = VectorGather(&X[i2]);

		const varying float Dist = Dists[i];
		const varying float Lambda = Lambdas[i];

		#pragma ignore warning(perf)
		const varying float Stiffness = StiffnessTable[StiffnessIndex];
		#pragma ignore warning(perf)
		const varying float DampingRatio = DampingTable[DampingIndex];

		varying float DLambda;
		const FVector3f Delta = GetXPBDSpringDampingDelta(P1, P2, X1, X2, InvM1, InvM2, Dt, Dist, Lambda, Stiffness, DampingRatio, DLambda);

		Lambdas[i] = Lambda + DLambda;
		
		VectorScatter(&PandInvM[i1], SetVector4( P1 + (InvM1 * Delta), InvM1 ));
		VectorScatter(&PandInvM[i2], SetVector4( P2 - (InvM2 * Delta), InvM2 ));
	}
}