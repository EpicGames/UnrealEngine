// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/XPBDStretchBiasElementConstraints.h"
#include "Chaos/SoftsSolverParticlesRange.h"
#include "Chaos/GraphColoring.h"
#include "Chaos/Framework/Parallel.h"
#include "Chaos/TriangleMesh.h"
#include "Chaos/Matrix.h"
#include "ChaosStats.h"

#if INTEL_ISPC
#include "XPBDStretchBiasElementConstraints.ispc.generated.h"
#endif

DECLARE_CYCLE_STAT(TEXT("Chaos XPBD Stretch Bias Constraint"), STAT_XPBD_StretchBias, STATGROUP_Chaos);

#if INTEL_ISPC && !UE_BUILD_SHIPPING
bool bChaos_XPBDStretchBiasElement_ISPC_Enabled = CHAOS_XPBD_STRETCH_BIAS_ELEMENT_ISPC_ENABLED_DEFAULT;
FAutoConsoleVariableRef CVarChaosXPBDStretchBiasISPCEnabled(TEXT("p.Chaos.XPBDStretchBias.ISPC"), bChaos_XPBDStretchBiasElement_ISPC_Enabled, TEXT("Whether to use ISPC optimizations in XPBD Stretch Bias constraints"));
static_assert(sizeof(ispc::FVector4f) == sizeof(Chaos::Softs::FPAndInvM), "sizeof(ispc::FVector4f) != sizeof(Chaos::Softs::FPAndInvM");
static_assert(sizeof(ispc::FIntVector) == sizeof(Chaos::TVec3<int32>), "sizeof(ispc::FIntVector) != sizeof(Chaos::TVec3<int32>");
//static_assert(sizeof(ispc::FVector2f) == sizeof(Chaos::Softs::FSolverVec2), "sizeof(ispc::FVector2f) != sizeof(Chaos::Softs::FSolverVec2");
static_assert(sizeof(ispc::FVector4f) == sizeof(Chaos::Softs::FSolverMatrix22), "sizeof(ispc::FVector2f) != sizeof(Chaos::Softs::FSolverMatrix22");
static_assert(sizeof(ispc::FVector3f) == sizeof(Chaos::Softs::FSolverVec3), "sizeof(ispc::FVector3f) != sizeof(Chaos::Softs::FSolverVec3");
#endif

namespace Chaos::Softs {

	Softs::FSolverReal StiffnessPaddingRatio = 0.f;

	FAutoConsoleVariableRef CVarClothStiffnessPaddingRatio(TEXT("p.Chaos.Cloth.StiffnessPaddingRatio"), StiffnessPaddingRatio, TEXT("stiffness padding for gauss seidel [def: 1]"));

// @todo(chaos): the parallel threshold (or decision to run parallel) should probably be owned by the solver and passed to the constraint container
static int32 Chaos_XPBDStretchBias_ParallelConstraintCount = 100;


 FXPBDStretchBiasElementConstraints::FXPBDStretchBiasElementConstraints(const FSolverParticlesRange& InParticles,
	const FTriangleMesh& TriangleMesh,
	const TArray<TVec3<FVec2f>>& FaceVertexUVs,
	const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
	const FCollectionPropertyConstFacade& PropertyCollection,
	bool bTrimKinematicConstraints)
	: ParticleOffset(0)
	, ParticleCount(InParticles.GetRangeSize())
	, StiffnessWarp(
		FSolverVec2(GetWeightedFloatXPBDAnisoStretchStiffnessWarp(PropertyCollection, MaxStiffness)),
		WeightMaps.FindRef(GetXPBDAnisoStretchStiffnessWarpString(PropertyCollection, XPBDAnisoStretchStiffnessWarpName.ToString())),
		TConstArrayView<TVec3<int32>>(Constraints),
		ParticleOffset,
		ParticleCount,
		FPBDStiffness::DefaultTableSize,
		FPBDStiffness::DefaultParameterFitBase,
		MaxStiffness)
	, StiffnessWeft(
		FSolverVec2(GetWeightedFloatXPBDAnisoStretchStiffnessWeft(PropertyCollection, MaxStiffness)),
		WeightMaps.FindRef(GetXPBDAnisoStretchStiffnessWeftString(PropertyCollection, XPBDAnisoStretchStiffnessWeftName.ToString())),
		TConstArrayView<TVec3<int32>>(Constraints),
		ParticleOffset,
		ParticleCount,
		FPBDStiffness::DefaultTableSize,
		FPBDStiffness::DefaultParameterFitBase,
		MaxStiffness)
	, StiffnessBias(
		FSolverVec2(GetWeightedFloatXPBDAnisoStretchStiffnessBias(PropertyCollection, MaxStiffness)),
		WeightMaps.FindRef(GetXPBDAnisoStretchStiffnessBiasString(PropertyCollection, XPBDAnisoStretchStiffnessBiasName.ToString())),
		TConstArrayView<TVec3<int32>>(Constraints),
		ParticleOffset,
		ParticleCount,
		FPBDStiffness::DefaultTableSize,
		FPBDStiffness::DefaultParameterFitBase,
		MaxStiffness)
	, DampingRatio(
		FSolverVec2(GetWeightedFloatXPBDAnisoStretchDamping(PropertyCollection, MinDamping)).ClampAxes(MinDamping, MaxDamping),
		WeightMaps.FindRef(GetXPBDAnisoStretchDampingString(PropertyCollection, XPBDAnisoStretchDampingName.ToString())),
		TConstArrayView<TVec3<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, WarpScale(
		FSolverVec2(GetWeightedFloatXPBDAnisoStretchWarpScale(PropertyCollection, DefaultWarpWeftScale)).ClampAxes(MinWarpWeftScale, MaxWarpWeftScale),
		WeightMaps.FindRef(GetXPBDAnisoStretchWarpScaleString(PropertyCollection, XPBDAnisoStretchWarpScaleName.ToString())),
		TConstArrayView<TVec3<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, WeftScale(
		FSolverVec2(GetWeightedFloatXPBDAnisoStretchWeftScale(PropertyCollection, DefaultWarpWeftScale)).ClampAxes(MinWarpWeftScale, MaxWarpWeftScale),
		WeightMaps.FindRef(GetXPBDAnisoStretchWeftScaleString(PropertyCollection, XPBDAnisoStretchWeftScaleName.ToString())),
		TConstArrayView<TVec3<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, XPBDAnisoStretchUse3dRestLengthsIndex(PropertyCollection)
	, XPBDAnisoStretchStiffnessWarpIndex(PropertyCollection)
	, XPBDAnisoStretchStiffnessWeftIndex(PropertyCollection)
	, XPBDAnisoStretchStiffnessBiasIndex(PropertyCollection)
	, XPBDAnisoStretchDampingIndex(PropertyCollection)
	, XPBDAnisoStretchWarpScaleIndex(PropertyCollection)
	, XPBDAnisoStretchWeftScaleIndex(PropertyCollection)
{
	Lambdas.Init(FSolverVec3(0.), Constraints.Num());
	InitConstraintsAndRestData(InParticles, TriangleMesh, FaceVertexUVs, GetXPBDAnisoStretchUse3dRestLengths(PropertyCollection, bDefaultUse3dRestLengths), bTrimKinematicConstraints);
	InitColor(InParticles);
}

FXPBDStretchBiasElementConstraints::FXPBDStretchBiasElementConstraints(const FSolverParticles& InParticles,
	int32 InParticleOffset,
	int32 InParticleCount,
	const FTriangleMesh& TriangleMesh,
	const TArray<TVec3<FVec2f>>& FaceVertexUVs,
	const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
	const FCollectionPropertyConstFacade& PropertyCollection,
	bool bTrimKinematicConstraints)
	: ParticleOffset(InParticleOffset)
	, ParticleCount(InParticleCount)
	, StiffnessWarp(
		FSolverVec2(GetWeightedFloatXPBDAnisoStretchStiffnessWarp(PropertyCollection, MaxStiffness)),
		WeightMaps.FindRef(GetXPBDAnisoStretchStiffnessWarpString(PropertyCollection, XPBDAnisoStretchStiffnessWarpName.ToString())),
		TConstArrayView<TVec3<int32>>(Constraints),
		ParticleOffset,
		ParticleCount,
		FPBDStiffness::DefaultTableSize,
		FPBDStiffness::DefaultParameterFitBase,
		MaxStiffness)
	, StiffnessWeft(
		FSolverVec2(GetWeightedFloatXPBDAnisoStretchStiffnessWeft(PropertyCollection, MaxStiffness)),
		WeightMaps.FindRef(GetXPBDAnisoStretchStiffnessWeftString(PropertyCollection, XPBDAnisoStretchStiffnessWeftName.ToString())),
		TConstArrayView<TVec3<int32>>(Constraints),
		ParticleOffset,
		ParticleCount,
		FPBDStiffness::DefaultTableSize,
		FPBDStiffness::DefaultParameterFitBase,
		MaxStiffness)
	, StiffnessBias(
		FSolverVec2(GetWeightedFloatXPBDAnisoStretchStiffnessBias(PropertyCollection, MaxStiffness)),
		WeightMaps.FindRef(GetXPBDAnisoStretchStiffnessBiasString(PropertyCollection, XPBDAnisoStretchStiffnessBiasName.ToString())),
		TConstArrayView<TVec3<int32>>(Constraints),
		ParticleOffset,
		ParticleCount,
		FPBDStiffness::DefaultTableSize,
		FPBDStiffness::DefaultParameterFitBase,
		MaxStiffness)
	, DampingRatio(
		FSolverVec2(GetWeightedFloatXPBDAnisoStretchDamping(PropertyCollection, MinDamping)).ClampAxes(MinDamping, MaxDamping),
		WeightMaps.FindRef(GetXPBDAnisoStretchDampingString(PropertyCollection, XPBDAnisoStretchDampingName.ToString())),
		TConstArrayView<TVec3<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, WarpScale(
		FSolverVec2(GetWeightedFloatXPBDAnisoStretchWarpScale(PropertyCollection, DefaultWarpWeftScale)).ClampAxes(MinWarpWeftScale, MaxWarpWeftScale),
		WeightMaps.FindRef(GetXPBDAnisoStretchWarpScaleString(PropertyCollection, XPBDAnisoStretchWarpScaleName.ToString())),
		TConstArrayView<TVec3<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, WeftScale(
		FSolverVec2(GetWeightedFloatXPBDAnisoStretchWeftScale(PropertyCollection, DefaultWarpWeftScale)).ClampAxes(MinWarpWeftScale, MaxWarpWeftScale),
		WeightMaps.FindRef(GetXPBDAnisoStretchWeftScaleString(PropertyCollection, XPBDAnisoStretchWeftScaleName.ToString())),
		TConstArrayView<TVec3<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, XPBDAnisoStretchUse3dRestLengthsIndex(PropertyCollection)
	, XPBDAnisoStretchStiffnessWarpIndex(PropertyCollection)
	, XPBDAnisoStretchStiffnessWeftIndex(PropertyCollection)
	, XPBDAnisoStretchStiffnessBiasIndex(PropertyCollection)
	, XPBDAnisoStretchDampingIndex(PropertyCollection)
	, XPBDAnisoStretchWarpScaleIndex(PropertyCollection)
	, XPBDAnisoStretchWeftScaleIndex(PropertyCollection)
{
	Lambdas.Init(FSolverVec3(0.), Constraints.Num());
	InitConstraintsAndRestData(InParticles, TriangleMesh, FaceVertexUVs, GetXPBDAnisoStretchUse3dRestLengths(PropertyCollection, bDefaultUse3dRestLengths), bTrimKinematicConstraints);
	InitColor(InParticles);
}

FXPBDStretchBiasElementConstraints::FXPBDStretchBiasElementConstraints(const FSolverParticles& InParticles,
	int32 InParticleOffset,
	int32 InParticleCount,
	const FTriangleMesh& TriangleMesh,
	const TArray<TVec3<FVec2f>>& FaceVertexUVs,
	const TConstArrayView<FRealSingle>& StiffnessWarpMultipliers,
	const TConstArrayView<FRealSingle>& StiffnessWeftMultipliers,
	const TConstArrayView<FRealSingle>& StiffnessBiasMultipliers,
	const TConstArrayView<FRealSingle>& DampingMultipliers,
	const TConstArrayView<FRealSingle>& WarpScaleMultipliers,
	const TConstArrayView<FRealSingle>& WeftScaleMultipliers,
	const FSolverVec2& InStiffnessWarp,
	const FSolverVec2& InStiffnessWeft,
	const FSolverVec2& InStiffnessBias,
	const FSolverVec2& InDampingRatio,
	const FSolverVec2& InWarpScale,
	const FSolverVec2& InWeftScale,
	bool bUse3dRestLengths,
	bool bTrimKinematicConstraints)
	: ParticleOffset(InParticleOffset)
	, ParticleCount(InParticleCount)
	, StiffnessWarp(
		InStiffnessWarp,
		StiffnessWarpMultipliers,
		TConstArrayView<TVec3<int32>>(Constraints),
		ParticleOffset,
		ParticleCount,
		FPBDStiffness::DefaultTableSize,
		FPBDStiffness::DefaultParameterFitBase,
		MaxStiffness)
	, StiffnessWeft(
		InStiffnessWeft,
		StiffnessWeftMultipliers,
		TConstArrayView<TVec3<int32>>(Constraints),
		ParticleOffset,
		ParticleCount,
		FPBDStiffness::DefaultTableSize,
		FPBDStiffness::DefaultParameterFitBase,
		MaxStiffness)
	, StiffnessBias(
		InStiffnessBias,
		StiffnessBiasMultipliers,
		TConstArrayView<TVec3<int32>>(Constraints),
		ParticleOffset,
		ParticleCount,
		FPBDStiffness::DefaultTableSize,
		FPBDStiffness::DefaultParameterFitBase,
		MaxStiffness)
	, DampingRatio(
		InDampingRatio,
		DampingMultipliers,
		TConstArrayView<TVec3<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, WarpScale(
		InWarpScale,
		WarpScaleMultipliers,
		TConstArrayView<TVec3<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, WeftScale(
		InWeftScale,
		WeftScaleMultipliers,
		TConstArrayView<TVec3<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	, XPBDAnisoStretchUse3dRestLengthsIndex(ForceInit)
	, XPBDAnisoStretchStiffnessWarpIndex(ForceInit)
	, XPBDAnisoStretchStiffnessWeftIndex(ForceInit)
	, XPBDAnisoStretchStiffnessBiasIndex(ForceInit)
	, XPBDAnisoStretchDampingIndex(ForceInit)
	, XPBDAnisoStretchWarpScaleIndex(ForceInit)
	, XPBDAnisoStretchWeftScaleIndex(ForceInit)
{
	Lambdas.Init(FSolverVec3(0.), Constraints.Num());
	InitConstraintsAndRestData(InParticles, TriangleMesh, FaceVertexUVs, bUse3dRestLengths, bTrimKinematicConstraints);
	InitColor(InParticles);
}

template<typename SolverParticlesOrRange>
void FXPBDStretchBiasElementConstraints::InitConstraintsAndRestData(const SolverParticlesOrRange& InParticles, const FTriangleMesh& TriangleMesh,
	const TArray<TVec3<FSolverVec2>>& FaceVertexUVs, const bool bUse3dRestLengths, const bool bTrimKinematicConstraints)
{
	const TArray<TVec3<int32>>& Elements = TriangleMesh.GetElements();
	check(Elements.Num() == FaceVertexUVs.Num());

	Constraints.Reserve(Elements.Num());
	RestStretchLengths.Reserve(Elements.Num());
	DeltaUVInverse.Reserve(Elements.Num());
	StiffnessScales.Reserve(Elements.Num());

	for (int32 ElemIdx = 0; ElemIdx < Elements.Num(); ++ElemIdx)
	{
		const TVec3<int32>& Constraint = Elements[ElemIdx];

		if (bTrimKinematicConstraints && InParticles.InvM(Constraint[0]) == (FSolverReal)0. && InParticles.InvM(Constraint[1]) == (FSolverReal)0. && InParticles.InvM(Constraint[2]) == (FSolverReal)0.)
		{
			continue;
		}

		Constraints.Add(Constraint);

		const FSolverVec2& UV0 = FaceVertexUVs[ElemIdx][0];
		const FSolverVec2& UV1 = FaceVertexUVs[ElemIdx][1];
		const FSolverVec2& UV2 = FaceVertexUVs[ElemIdx][2];

		const FSolverVec2 UV01 = UV1 - UV0;
		const FSolverVec2 UV02 = UV2 - UV0;


		if (bUse3dRestLengths)
		{
			const FSolverVec2 DeltaUNormalized = FSolverVec2(UV01[0], UV02[0]).GetSafeNormal();
			const FSolverVec2 DeltaVNormalized = FSolverVec2(UV01[1], UV02[1]).GetSafeNormal();
			const FSolverReal DeltaUVDet = DeltaUNormalized[0] * DeltaVNormalized[1] - DeltaVNormalized[0] * DeltaUNormalized[1];

			const FSolverMatrix22& DeltaUVInv = FMath::Abs(DeltaUVDet) < UE_SMALL_NUMBER ? DeltaUVInverse.Add_GetRef(FSolverMatrix22(1.f, 0.f, 0.f, 1.f)) :
				DeltaUVInverse.Add_GetRef(FSolverMatrix22(DeltaVNormalized[1] / DeltaUVDet, -DeltaVNormalized[0] / DeltaUVDet, -DeltaUNormalized[1] / DeltaUVDet, DeltaUNormalized[0] / DeltaUVDet));

			const FSolverVec3& X0 = InParticles.GetX(Constraint[0]);
			const FSolverVec3& X1 = InParticles.GetX(Constraint[1]);
			const FSolverVec3& X2 = InParticles.GetX(Constraint[2]);
			const FSolverVec3 X01 = X1 - X0;
			const FSolverVec3 X02 = X2 - X0; 
			const FSolverVec3 DXDu = X01 * DeltaUVInv.M[0] + X02 * DeltaUVInv.M[1];
			const FSolverVec3 DXDv = X01 * DeltaUVInv.M[2] + X02 * DeltaUVInv.M[3];

			auto SafeRecip = [](const FSolverReal Len, const FSolverReal Fallback)
			{
				if (Len > UE_SMALL_NUMBER)
				{
					return 1.f / Len;
				}
				return Fallback;
			};
			const FSolverReal OneOverDXDuLen = SafeRecip(DXDu.Length(), 0.f);
			const FSolverReal OneOverDXDvLen = SafeRecip(DXDv.Length(), 0.f);

			const FSolverVec3 DXDuNormalized = DXDu * OneOverDXDuLen;
			const FSolverVec3 DXDvNormalized = DXDv * OneOverDXDvLen;

			const FSolverReal ThetaRestStretch = FSolverVec3::DotProduct(DXDuNormalized, DXDvNormalized);

			const FSolverVec3& RestStretch = RestStretchLengths.Add_GetRef(FSolverVec3(DXDu.Length(), DXDv.Length(), ThetaRestStretch));
			
			const FSolverReal StiffnessScaleU = RestStretch[0] < UE_SMALL_NUMBER ? 1.f : 1.f / RestStretch[0];
			const FSolverReal StiffnessScaleV = RestStretch[1] < UE_SMALL_NUMBER ? 1.f : 1.f / RestStretch[1];
			const FSolverReal BiasScale = FMath::Sqrt((FSolverReal).5 * FSolverVec3::CrossProduct(X01, X02).Length()); // Sqrt of triangle area

			StiffnessScales.Add(FSolverVec3(StiffnessScaleU, StiffnessScaleV, BiasScale));
		}
		else
		{
			const FSolverVec2 DeltaUUnnormalized = FSolverVec2(UV01[0], UV02[0]);
			const FSolverVec2 DeltaVUnnormalized = FSolverVec2(UV01[1], UV02[1]);
			const FSolverReal DeltaUVDet = DeltaUUnnormalized[0] * DeltaVUnnormalized[1] - DeltaVUnnormalized[0] * DeltaUUnnormalized[1];
			const FSolverMatrix22& DeltaUVInv = FMath::Abs(DeltaUVDet) < UE_SMALL_NUMBER ? DeltaUVInverse.Add_GetRef(FSolverMatrix22(1.f, 0.f, 0.f, 1.f)) :
				DeltaUVInverse.Add_GetRef(FSolverMatrix22(DeltaVUnnormalized[1] / DeltaUVDet, -DeltaVUnnormalized[0] / DeltaUVDet, -DeltaUUnnormalized[1] / DeltaUVDet, DeltaUUnnormalized[0] / DeltaUVDet));
			RestStretchLengths.Add(FSolverVec3(1.f, 1.f, 0.f));

			const FSolverReal DeltaULen = DeltaUUnnormalized.Length();
			const FSolverReal DeltaVLen = DeltaVUnnormalized.Length();

			const FSolverReal StiffnessScaleU = DeltaULen < UE_SMALL_NUMBER ? 1.f : 1.f / DeltaULen;
			const FSolverReal StiffnessScaleV = DeltaVLen < UE_SMALL_NUMBER ? 1.f : 1.f / DeltaVLen;
			const FSolverReal BiasScale = FMath::Sqrt((FSolverReal).5 * FMath::Abs(UV01[0] * UV02[1] - UV01[1] * UV02[0]));
			StiffnessScales.Add(FSolverVec3(StiffnessScaleU, StiffnessScaleV, BiasScale));
		}
	}
}

template<typename SolverParticlesOrRange>
void FXPBDStretchBiasElementConstraints::InitColor(const SolverParticlesOrRange& InParticles)
{
	ConstraintsPerColorStartIndex.Reset();

	// In dev builds we always color so we can tune the system without restarting. See Apply()
#if UE_BUILD_SHIPPING || UE_BUILD_TEST
	if (Constraints.Num() > Chaos_XPBDStretchBias_ParallelConstraintCount)
#endif
	{
		const TArray<TArray<int32>> ConstraintsPerColor = FGraphColoring::ComputeGraphColoringParticlesOrRange(Constraints, InParticles, ParticleOffset, ParticleOffset + ParticleCount);

		// Reorder constraints based on color so each array in ConstraintsPerColor contains contiguous elements.
		TArray<TVec3<int32>> ReorderedConstraints;
		TArray<FSolverVec3> ReorderedRestStretchLengths;
		TArray<FSolverMatrix22> ReorderedDeltaUVInverse;
		TArray<FSolverVec3> ReorderedStiffnessScales;
		TArray<int32> OrigToReorderedIndices; // used to reorder stiffness indices
		ReorderedConstraints.SetNumUninitialized(Constraints.Num());
		ReorderedRestStretchLengths.SetNumUninitialized(Constraints.Num());
		ReorderedDeltaUVInverse.SetNumUninitialized(Constraints.Num());
		ReorderedStiffnessScales.SetNumUninitialized(Constraints.Num());
		OrigToReorderedIndices.SetNumUninitialized(Constraints.Num());

		ConstraintsPerColorStartIndex.Reset(ConstraintsPerColor.Num() + 1);

		int32 ReorderedIndex = 0;
		for (const TArray<int32>& ConstraintsBatch : ConstraintsPerColor)
		{
			ConstraintsPerColorStartIndex.Add(ReorderedIndex);
			for (const int32& BatchConstraint : ConstraintsBatch)
			{
				const int32 OrigIndex = BatchConstraint;
				ReorderedConstraints[ReorderedIndex] = Constraints[OrigIndex];
				ReorderedRestStretchLengths[ReorderedIndex] = RestStretchLengths[OrigIndex];
				ReorderedDeltaUVInverse[ReorderedIndex] = DeltaUVInverse[OrigIndex];
				ReorderedStiffnessScales[ReorderedIndex] = StiffnessScales[OrigIndex];
				OrigToReorderedIndices[OrigIndex] = ReorderedIndex;

				++ReorderedIndex;
			}
		}
		check(ReorderedIndex == Constraints.Num());
		ConstraintsPerColorStartIndex.Add(ReorderedIndex);

		Constraints = MoveTemp(ReorderedConstraints);
		RestStretchLengths = MoveTemp(ReorderedRestStretchLengths);
		DeltaUVInverse = MoveTemp(ReorderedDeltaUVInverse);
		StiffnessScales = MoveTemp(ReorderedStiffnessScales);
		StiffnessWarp.ReorderIndices(OrigToReorderedIndices);
		StiffnessWeft.ReorderIndices(OrigToReorderedIndices);
		StiffnessBias.ReorderIndices(OrigToReorderedIndices);
		DampingRatio.ReorderIndices(OrigToReorderedIndices);
	}

	// Ensure we have valid color indices
	if (ConstraintsPerColorStartIndex.Num() < 2)
	{
		ConstraintsPerColorStartIndex.Reset(2);
		ConstraintsPerColorStartIndex.Add(0);
		ConstraintsPerColorStartIndex.Add(Constraints.Num());
	}
}

void FXPBDStretchBiasElementConstraints::SetProperties(
	const FCollectionPropertyConstFacade& PropertyCollection,
	const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps)
{
	if (IsXPBDAnisoStretchStiffnessWarpMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedValue(GetWeightedFloatXPBDAnisoStretchStiffnessWarp(PropertyCollection));
		if (IsXPBDAnisoStretchStiffnessWarpStringDirty(PropertyCollection))
		{
			const FString& WeightMapName = GetXPBDAnisoStretchStiffnessWarpString(PropertyCollection);
			StiffnessWarp = FPBDStiffness(
				WeightedValue,
				WeightMaps.FindRef(WeightMapName),
				TConstArrayView<TVec3<int32>>(Constraints),
				ParticleOffset,
				ParticleCount,
				FPBDStiffness::DefaultTableSize,
				FPBDStiffness::DefaultParameterFitBase,
				MaxStiffness);
		}
		else
		{
			StiffnessWarp.SetWeightedValue(WeightedValue, MaxStiffness);
		}
	}
	if (IsXPBDAnisoStretchStiffnessWeftMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedValue(GetWeightedFloatXPBDAnisoStretchStiffnessWeft(PropertyCollection));
		if (IsXPBDAnisoStretchStiffnessWeftStringDirty(PropertyCollection))
		{
			const FString& WeightMapName = GetXPBDAnisoStretchStiffnessWeftString(PropertyCollection);
			StiffnessWeft = FPBDStiffness(
				WeightedValue,
				WeightMaps.FindRef(WeightMapName),
				TConstArrayView<TVec3<int32>>(Constraints),
				ParticleOffset,
				ParticleCount,
				FPBDStiffness::DefaultTableSize,
				FPBDStiffness::DefaultParameterFitBase,
				MaxStiffness);
		}
		else
		{
			StiffnessWeft.SetWeightedValue(WeightedValue, MaxStiffness);
		}
	}
	if (IsXPBDAnisoStretchStiffnessBiasMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedValue(GetWeightedFloatXPBDAnisoStretchStiffnessBias(PropertyCollection));
		if (IsXPBDAnisoStretchStiffnessBiasStringDirty(PropertyCollection))
		{
			const FString& WeightMapName = GetXPBDAnisoStretchStiffnessBiasString(PropertyCollection);
			StiffnessBias = FPBDStiffness(
				WeightedValue,
				WeightMaps.FindRef(WeightMapName),
				TConstArrayView<TVec3<int32>>(Constraints),
				ParticleOffset,
				ParticleCount,
				FPBDStiffness::DefaultTableSize,
				FPBDStiffness::DefaultParameterFitBase,
				MaxStiffness);
		}
		else
		{
			StiffnessBias.SetWeightedValue(WeightedValue, MaxStiffness);
		}
	}
	if (IsXPBDAnisoStretchDampingMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedValue = FSolverVec2(GetWeightedFloatXPBDAnisoStretchDamping(PropertyCollection)).ClampAxes(MinDamping, MaxDamping);
		if (IsXPBDAnisoStretchDampingStringDirty(PropertyCollection))
		{
			const FString& WeightMapName = GetXPBDAnisoStretchDampingString(PropertyCollection);
			DampingRatio = FPBDWeightMap(
				WeightedValue,
				WeightMaps.FindRef(WeightMapName),
				TConstArrayView<TVec3<int32>>(Constraints),
				ParticleOffset,
				ParticleCount);
		}
		else
		{
			DampingRatio.SetWeightedValue(WeightedValue);
		}
	}
	if (IsXPBDAnisoStretchWarpScaleMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedValue = FSolverVec2(GetWeightedFloatXPBDAnisoStretchWarpScale(PropertyCollection)).ClampAxes(MinWarpWeftScale, MaxWarpWeftScale);
		if (IsXPBDAnisoStretchWarpScaleStringDirty(PropertyCollection))
		{
			const FString& WeightMapName = GetXPBDAnisoStretchWarpScaleString(PropertyCollection);
			WarpScale = FPBDWeightMap(
				WeightedValue,
				WeightMaps.FindRef(WeightMapName),
				TConstArrayView<TVec3<int32>>(Constraints),
				ParticleOffset,
				ParticleCount);
		}
		else
		{
			WarpScale.SetWeightedValue(WeightedValue);
		}
	}
	if (IsXPBDAnisoStretchWeftScaleMutable(PropertyCollection))
	{
		const FSolverVec2 WeightedValue = FSolverVec2(GetWeightedFloatXPBDAnisoStretchWeftScale(PropertyCollection)).ClampAxes(MinWarpWeftScale, MaxWarpWeftScale);
		if (IsXPBDAnisoStretchWeftScaleStringDirty(PropertyCollection))
		{
			const FString& WeightMapName = GetXPBDAnisoStretchWeftScaleString(PropertyCollection);
			WeftScale = FPBDWeightMap(
				WeightedValue,
				WeightMaps.FindRef(WeightMapName),
				TConstArrayView<TVec3<int32>>(Constraints),
				ParticleOffset,
				ParticleCount);
		}
		else
		{
			WeftScale.SetWeightedValue(WeightedValue);
		}
	}
}

template<typename SolverParticlesOrRange>
void FXPBDStretchBiasElementConstraints::Apply(SolverParticlesOrRange& Particles, const FSolverReal Dt) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FXPBDStretchBiasElementConstraints_Apply);
	SCOPE_CYCLE_COUNTER(STAT_XPBD_StretchBias);

	const bool StiffnessWarpHasWeightMap = StiffnessWarp.HasWeightMap();
	const bool StiffnessWeftHasWeightMap = StiffnessWeft.HasWeightMap();
	const bool StiffnessBiasHasWeightMap = StiffnessBias.HasWeightMap();
	const bool DampingHasWeightMap = DampingRatio.HasWeightMap();
	const bool WarpScaleHasWeightMap = WarpScale.HasWeightMap();
	const bool WeftScaleHasWeightMap = WeftScale.HasWeightMap();

	const FSolverReal StiffnessWarpNoMap = (FSolverReal)StiffnessWarp;
	const FSolverReal StiffnessWeftNoMap = (FSolverReal)StiffnessWeft;
	const FSolverReal StiffnessBiasNoMap = (FSolverReal)StiffnessBias;
	const FSolverReal DampingNoMap = (FSolverReal)DampingRatio;
	const FSolverReal WarpScaleNoMap = (FSolverReal)WarpScale;
	const FSolverReal WeftScaleNoMap = (FSolverReal)WeftScale;

	if (ConstraintsPerColorStartIndex.Num() > 1 && Constraints.Num() > Chaos_XPBDStretchBias_ParallelConstraintCount)
	{
		const int32 ConstraintColorNum = ConstraintsPerColorStartIndex.Num() - 1;
		if (!StiffnessWarpHasWeightMap && !StiffnessWeftHasWeightMap && !StiffnessBiasHasWeightMap && !DampingHasWeightMap && !WarpScaleHasWeightMap && !WeftScaleHasWeightMap)
		{
			const FSolverVec3 ExpStiffnessValue(StiffnessWeftNoMap, StiffnessWarpNoMap, StiffnessBiasNoMap);
			if (ExpStiffnessValue.Max() < MinStiffness)
			{
				return;
			}

#if INTEL_ISPC
			if (bRealTypeCompatibleWithISPC && bChaos_XPBDStretchBiasElement_ISPC_Enabled)
			{
				if (DampingNoMap > 0)
				{
					for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
					{
						const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
						const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
						ispc::ApplyXPBDStretchBiasConstraintsWithDamping(
							(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
							(const ispc::FVector3f*)Particles.XArray().GetData(),
							(ispc::FIntVector*)&Constraints.GetData()[ColorStart],
							(ispc::FVector3f*)&RestStretchLengths.GetData()[ColorStart],
							(ispc::FVector4f*)&DeltaUVInverse.GetData()[ColorStart],
							(ispc::FVector3f*)&StiffnessScales.GetData()[ColorStart],
							(ispc::FVector3f*)&Lambdas.GetData()[ColorStart],
							Dt,
							MinStiffness,
							reinterpret_cast<const ispc::FVector3f&>(ExpStiffnessValue),
							DampingNoMap,
							WarpScaleNoMap,
							WeftScaleNoMap,
							ColorSize);
					}

				}
				else
				{
					for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
					{
						const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
						const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
						ispc::ApplyXPBDStretchBiasConstraints(
							(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
							(ispc::FIntVector*)&Constraints.GetData()[ColorStart],
							(ispc::FVector3f*)&RestStretchLengths.GetData()[ColorStart],
							(ispc::FVector4f*)&DeltaUVInverse.GetData()[ColorStart],
							(ispc::FVector3f*)&StiffnessScales.GetData()[ColorStart],
							(ispc::FVector3f*)&Lambdas.GetData()[ColorStart],
							Dt,
							MinStiffness,
							reinterpret_cast<const ispc::FVector3f&>(ExpStiffnessValue),
							WarpScaleNoMap,
							WeftScaleNoMap,
							ColorSize);
					}
				}
			}
			else
#endif
			{
				for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
				{
					const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
					const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
					PhysicsParallelFor(ColorSize, [&](const int32 Index)
					{
						const int32 ConstraintIndex = ColorStart + Index;
						ApplyHelper(Particles, Dt, ConstraintIndex, ExpStiffnessValue, DampingNoMap, WarpScaleNoMap, WeftScaleNoMap);
					});
				}
			}			
		}
		else // has weight maps
		{

#if INTEL_ISPC
			if (bRealTypeCompatibleWithISPC && bChaos_XPBDStretchBiasElement_ISPC_Enabled)
			{
				if (DampingHasWeightMap || DampingNoMap > 0)
				{
					for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
					{
						const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
						const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
						ispc::ApplyXPBDStretchBiasConstraintsWithDampingAndMaps(
							(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
							(const ispc::FVector3f*)Particles.XArray().GetData(),
							(ispc::FIntVector*)&Constraints.GetData()[ColorStart],
							(ispc::FVector3f*)&RestStretchLengths.GetData()[ColorStart],
							(ispc::FVector4f*)&DeltaUVInverse.GetData()[ColorStart],
							(ispc::FVector3f*)&StiffnessScales.GetData()[ColorStart],
							(ispc::FVector3f*)&Lambdas.GetData()[ColorStart],
							Dt,
							MinStiffness,
							StiffnessWarpHasWeightMap,
							StiffnessWarpHasWeightMap ? &StiffnessWarp.GetIndices().GetData()[ColorStart] : nullptr,
							&StiffnessWarp.GetTable().GetData()[0],
							StiffnessWeftHasWeightMap,
							StiffnessWeftHasWeightMap ? &StiffnessWeft.GetIndices().GetData()[ColorStart] : nullptr,
							&StiffnessWeft.GetTable().GetData()[0],
							StiffnessBiasHasWeightMap,
							StiffnessBiasHasWeightMap ? &StiffnessBias.GetIndices().GetData()[ColorStart] : nullptr,
							&StiffnessBias.GetTable().GetData()[0],
							DampingHasWeightMap,
							DampingHasWeightMap ? &DampingRatio.GetIndices().GetData()[ColorStart] : nullptr,
							&DampingRatio.GetTable().GetData()[0],
							WarpScaleHasWeightMap,
							WarpScaleHasWeightMap ? &WarpScale.GetIndices().GetData()[ColorStart] : nullptr,
							&WarpScale.GetTable().GetData()[0],
							WeftScaleHasWeightMap,
							WeftScaleHasWeightMap ? &WeftScale.GetIndices().GetData()[ColorStart] : nullptr,
							&WeftScale.GetTable().GetData()[0],
							ColorSize);
					}

				}
				else
				{
					for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
					{
						const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
						const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
						ispc::ApplyXPBDStretchBiasConstraintsWithMaps(
							(ispc::FVector4f*)Particles.GetPAndInvM().GetData(),
							(ispc::FIntVector*)&Constraints.GetData()[ColorStart],
							(ispc::FVector3f*)&RestStretchLengths.GetData()[ColorStart],
							(ispc::FVector4f*)&DeltaUVInverse.GetData()[ColorStart],
							(ispc::FVector3f*)&StiffnessScales.GetData()[ColorStart],
							(ispc::FVector3f*)&Lambdas.GetData()[ColorStart],
							Dt,
							MinStiffness,
							StiffnessWarpHasWeightMap,
							StiffnessWarpHasWeightMap ? &StiffnessWarp.GetIndices().GetData()[ColorStart] : nullptr,
							&StiffnessWarp.GetTable().GetData()[0],
							StiffnessWeftHasWeightMap,
							StiffnessWeftHasWeightMap ? &StiffnessWeft.GetIndices().GetData()[ColorStart] : nullptr,
							&StiffnessWeft.GetTable().GetData()[0],
							StiffnessBiasHasWeightMap,
							StiffnessBiasHasWeightMap ? &StiffnessBias.GetIndices().GetData()[ColorStart] : nullptr,
							&StiffnessBias.GetTable().GetData()[0],
							WarpScaleHasWeightMap,
							WarpScaleHasWeightMap ? &WarpScale.GetIndices().GetData()[ColorStart] : nullptr,
							&WarpScale.GetTable().GetData()[0],
							WeftScaleHasWeightMap,
							WeftScaleHasWeightMap ? &WeftScale.GetIndices().GetData()[ColorStart] : nullptr,
							&WeftScale.GetTable().GetData()[0],
							ColorSize);
					}
				}

			}
			else
#endif
			{
				for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
				{
					const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
					const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
					PhysicsParallelFor(ColorSize, [&](const int32 Index)
					{
						const int32 ConstraintIndex = ColorStart + Index;
						const FSolverReal ExpStiffnessWarpValue = StiffnessWarpHasWeightMap ? StiffnessWarp[ConstraintIndex] : StiffnessWarpNoMap;
						const FSolverReal ExpStiffnessWeftValue = StiffnessWeftHasWeightMap ? StiffnessWeft[ConstraintIndex] : StiffnessWeftNoMap;
						const FSolverReal ExpStiffnessBiasValue = StiffnessBiasHasWeightMap ? StiffnessBias[ConstraintIndex] : StiffnessBiasNoMap;
						const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
						const FSolverReal WarpScaleValue = WarpScaleHasWeightMap ? WarpScale[ConstraintIndex] : WarpScaleNoMap;
						const FSolverReal WeftScaleValue = WeftScaleHasWeightMap ? WeftScale[ConstraintIndex] : WeftScaleNoMap;
						ApplyHelper(Particles, Dt, ConstraintIndex, FSolverVec3(ExpStiffnessWeftValue, ExpStiffnessWarpValue, ExpStiffnessBiasValue),DampingRatioValue, WarpScaleValue, WeftScaleValue);
					});
				}
			}
		}
	}
	else
	{
		if (!StiffnessWarpHasWeightMap && !StiffnessWeftHasWeightMap && !StiffnessBiasHasWeightMap && !DampingHasWeightMap)
		{
			const FSolverVec3 ExpStiffnessValue(StiffnessWarpNoMap, StiffnessWeftNoMap, StiffnessBiasNoMap);
			if (ExpStiffnessValue.Max() < MinStiffness)
			{
				return;
			}
			for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
			{
				ApplyHelper(Particles, Dt, ConstraintIndex, ExpStiffnessValue, DampingNoMap, WarpScaleNoMap, WeftScaleNoMap);
			}
		}
		else
		{
			for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
			{
				const FSolverReal ExpStiffnessWarpValue = StiffnessWarpHasWeightMap ? StiffnessWarp[ConstraintIndex] : StiffnessWarpNoMap;
				const FSolverReal ExpStiffnessWeftValue = StiffnessWeftHasWeightMap ? StiffnessWeft[ConstraintIndex] : StiffnessWeftNoMap;
				const FSolverReal ExpStiffnessBiasValue = StiffnessBiasHasWeightMap ? StiffnessBias[ConstraintIndex] : StiffnessBiasNoMap;
				const FSolverReal DampingRatioValue = DampingHasWeightMap ? DampingRatio[ConstraintIndex] : DampingNoMap;
				const FSolverReal WarpScaleValue = WarpScaleHasWeightMap ? WarpScale[ConstraintIndex] : WarpScaleNoMap;
				const FSolverReal WeftScaleValue = WeftScaleHasWeightMap ? WeftScale[ConstraintIndex] : WeftScaleNoMap;
				ApplyHelper(Particles, Dt, ConstraintIndex, FSolverVec3(ExpStiffnessWeftValue, ExpStiffnessWarpValue, ExpStiffnessBiasValue), DampingRatioValue, WarpScaleValue, WeftScaleValue);
			}
		}
	}
}
template CHAOS_API void FXPBDStretchBiasElementConstraints::Apply(FSolverParticles& Particles, const FSolverReal Dt) const;
template CHAOS_API void FXPBDStretchBiasElementConstraints::Apply(FSolverParticlesRange& Particles, const FSolverReal Dt) const;

void FXPBDStretchBiasElementConstraints::CalculateUVStretch(const int32 ConstraintIndex, const FSolverVec3& P0, const FSolverVec3& P1, const FSolverVec3& P2, FSolverVec3& DXDu, FSolverVec3& DXDv) const
{
	const FSolverMatrix22& DeltaUVInv = DeltaUVInverse[ConstraintIndex];

	const FSolverVec3 P01 = P1 - P0;
	const FSolverVec3 P02 = P2 - P0;
	DXDu = P01 * DeltaUVInv.M[0] + P02 * DeltaUVInv.M[1];
	DXDv = P01 * DeltaUVInv.M[2] + P02 * DeltaUVInv.M[3];
}

static FSolverReal CalcDLambda(const FSolverReal StiffnessValue, const FSolverReal Dt, const FSolverReal Lambda, const FSolverReal Damping,
	const FSolverReal InvM1, const FSolverReal InvM2, const FSolverReal InvM3,
	const FSolverVec3& V1TimesDt, const FSolverVec3& V2TimesDt, const FSolverVec3& V3TimesDt,
	const FSolverReal C, const FSolverVec3& dC_dX1, const FSolverVec3& dC_dX2, const FSolverVec3& dC_dX3)
{
	if (StiffnessValue < FXPBDStretchBiasElementConstraints::MinStiffness)
	{
		return (FSolverReal)0.f;
	}
	const FSolverReal Alpha = (FSolverReal)1.f / (StiffnessValue * Dt * Dt);
	const FSolverReal Gamma = Alpha * Damping * Dt;
	const FSolverReal DampingTerm = Gamma * (FSolverVec3::DotProduct(V1TimesDt, dC_dX1) + FSolverVec3::DotProduct(V2TimesDt, dC_dX2) + FSolverVec3::DotProduct(V3TimesDt, dC_dX3));
	const FSolverReal Denom = ((FSolverReal)1.f + Gamma) * (InvM1 * dC_dX1.SizeSquared() + InvM2 * dC_dX2.SizeSquared() + InvM3 * dC_dX3.SizeSquared()) + Alpha;
	return (C - Alpha * Lambda + DampingTerm) / Denom;
}

template<typename SolverParticlesOrRange>
void FXPBDStretchBiasElementConstraints::ApplyHelper(SolverParticlesOrRange& Particles, const FSolverReal Dt, const int32 ConstraintIndex, const FSolverVec3& ExpStiffnessValue, const FSolverReal DampingRatioValue, const FSolverReal WarpScaleValue, const FSolverReal WeftScaleValue) const
{
	const TVec3<int32>& Constraint = Constraints[ConstraintIndex];
	const int32 Idx0 = Constraint[0];
	const int32 Idx1 = Constraint[1];
	const int32 Idx2 = Constraint[2];
	if (Particles.InvM(Idx0) == (FSolverReal)0. && Particles.InvM(Idx1) == (FSolverReal)0. && Particles.InvM(Idx2) == (FSolverReal)0.)
	{
		return;
	}

	const FSolverReal CombinedInvMass = Particles.InvM(Idx0) + Particles.InvM(Idx1) + Particles.InvM(Idx2);

	FSolverVec3 DXDu, DXDv;
	CalculateUVStretch(ConstraintIndex, Particles.P(Idx0), Particles.P(Idx1), Particles.P(Idx2), DXDu, DXDv);

	const FSolverReal DXDuLength = DXDu.Length();
	const FSolverReal DXDvLength = DXDv.Length();
	auto SafeRecip = [](const FSolverReal Len, const FSolverReal Fallback)
	{
		if (Len > UE_SMALL_NUMBER)
		{
			return 1.f / Len;
		}
		return Fallback;
	};
	const FSolverReal OneOverDXDuLen = SafeRecip(DXDuLength, 0.f);
	const FSolverReal OneOverDXDvLen = SafeRecip(DXDvLength, 0.f);

	const FSolverVec3 DXDuNormalized = DXDu * OneOverDXDuLen;
	const FSolverVec3 DXDvNormalized = DXDv * OneOverDXDvLen;

	// constraints
	const FSolverReal Cu = (DXDuLength - RestStretchLengths[ConstraintIndex][0]*WeftScaleValue); // stretch in weft direction
	const FSolverReal Cv = (DXDvLength - RestStretchLengths[ConstraintIndex][1]*WarpScaleValue); // stretch in warp direction
	const FSolverReal Cs = FSolverVec3::DotProduct(DXDuNormalized, DXDvNormalized) - RestStretchLengths[ConstraintIndex][2];

	const FSolverMatrix22& DeltaUVInv = DeltaUVInverse[ConstraintIndex];

	const FSolverVec3 GradCu1 = DXDuNormalized * DeltaUVInv.M[0];
	const FSolverVec3 GradCu2 = DXDuNormalized * DeltaUVInv.M[1];
	const FSolverVec3 GradCu0 = -GradCu1 - GradCu2;
	const FSolverVec3 GradCv1 = DXDvNormalized * DeltaUVInv.M[2];
	const FSolverVec3 GradCv2 = DXDvNormalized * DeltaUVInv.M[3];
	const FSolverVec3 GradCv0 = -GradCv1 - GradCv2;
	const FSolverVec3 GradCs1 = (DXDvNormalized * OneOverDXDuLen * DeltaUVInv.M[0] + DXDuNormalized * OneOverDXDvLen * DeltaUVInv.M[2]);
	const FSolverVec3 GradCs2 = (DXDvNormalized * OneOverDXDuLen * DeltaUVInv.M[1] + DXDuNormalized * OneOverDXDvLen * DeltaUVInv.M[3]);
	const FSolverVec3 GradCs0 = -GradCs1 - GradCs2;

	const FSolverVec3 V0TimesDt = Particles.GetP(Idx0) - Particles.GetX(Idx0);
	const FSolverVec3 V1TimesDt = Particles.GetP(Idx1) - Particles.GetX(Idx1);
	const FSolverVec3 V2TimesDt = Particles.GetP(Idx2) - Particles.GetX(Idx2);

	// These scale factors make everything resolution independent
	const FSolverVec3 FinalStiffnesses = StiffnessScales[ConstraintIndex] * ExpStiffnessValue;

	const FSolverVec3 Damping = (FSolverReal)2.f * DampingRatioValue * FMath::InvSqrt(CombinedInvMass) *
		FSolverVec3(FMath::Sqrt(FinalStiffnesses[0]), FMath::Sqrt(FinalStiffnesses[1]), FMath::Sqrt(FinalStiffnesses[2]));

	FSolverVec3& Lambda = Lambdas[ConstraintIndex];
	FSolverVec3 DLambda;
	DLambda[0] = CalcDLambda(FinalStiffnesses[0], Dt, Lambda[0], Damping[0], Particles.InvM(Idx0), Particles.InvM(Idx1), Particles.InvM(Idx2),
		V0TimesDt, V1TimesDt, V2TimesDt, Cu, GradCu0, GradCu1, GradCu2);
	DLambda[1] = CalcDLambda(FinalStiffnesses[1], Dt, Lambda[1], Damping[1], Particles.InvM(Idx0), Particles.InvM(Idx1), Particles.InvM(Idx2),
		V0TimesDt, V1TimesDt, V2TimesDt, Cv, GradCv0, GradCv1, GradCv2);
	DLambda[2] = CalcDLambda(FinalStiffnesses[2], Dt, Lambda[2], Damping[2], Particles.InvM(Idx0), Particles.InvM(Idx1), Particles.InvM(Idx2),
		V0TimesDt, V1TimesDt, V2TimesDt, Cs, GradCs0, GradCs1, GradCs2);

	Particles.P(Idx0) -= Particles.InvM(Idx0) * (DLambda[0] * GradCu0 + DLambda[1] * GradCv0 + DLambda[2] * GradCs0);
	Particles.P(Idx1) -= Particles.InvM(Idx1) * (DLambda[0] * GradCu1 + DLambda[1] * GradCv1 + DLambda[2] * GradCs1);
	Particles.P(Idx2) -= Particles.InvM(Idx2) * (DLambda[0] * GradCu2 + DLambda[1] * GradCv2 + DLambda[2] * GradCs2);

	Lambda += DLambda;
}


void FXPBDStretchBiasElementConstraints::InitializeDmInvAndMeasures(const FSolverParticlesRange& Particles)
{
	Measure.SetNum(Constraints.Num());
	DmInverse.Init(PMatrix<FSolverReal, 2, 2>(0.f, 0.f, 0.f), Constraints.Num());
	DmArray.Init(PMatrix<FSolverReal, 2, 2>(0.f, 0.f, 0.f),Constraints.Num());
	RestDmArray.Init(PMatrix<FSolverReal, 3, 2>(0.f),Constraints.Num());
	for (int32 ElemIdx = 0; ElemIdx < Constraints.Num(); ElemIdx++)
	{
		const TVec3<FSolverReal> X1X0 = Particles.GetX(Constraints[ElemIdx][1]) - Particles.GetX(Constraints[ElemIdx][0]);
		const TVec3<FSolverReal> X2X0 = Particles.GetX(Constraints[ElemIdx][2]) - Particles.GetX(Constraints[ElemIdx][0]);
		PMatrix<FSolverReal, 3, 2> RestDm(0.f);
		PMatrix<FSolverReal, 2, 2> Dm(0.f, 0.f, 0.f);
		Dm.M[0] = X1X0.Size();
		Dm.M[2] = X1X0.Dot(X2X0) / Dm.M[0];
		Dm.M[3] = Chaos::TVector<FSolverReal, 3>::CrossProduct(X1X0, X2X0).Size() / Dm.M[0];
		Measure[ElemIdx] = Chaos::TVector<FSolverReal, 3>::CrossProduct(X1X0, X2X0).Size() / 2.f;
		ensureMsgf(Measure[ElemIdx] > 0.f, TEXT("Degenerate triangle detected"));

		const PMatrix<FSolverReal, 2, 2> DmInv = Dm.Inverse();
		DmInverse[ElemIdx] = DmInv;
		DmArray[ElemIdx] = Dm;
		RestDmArray[ElemIdx] = Chaos::PMatrix<FSolverReal, 3, 2>(X1X0, X2X0);
	}

	bDmInitialized = true;
}

void FXPBDStretchBiasElementConstraints::InitializeDmInvAndMeasures(const FSolverParticles& Particles)
{
	InitializeDmInvAndMeasures(FSolverParticlesRange(Particles));
}

void FXPBDStretchBiasElementConstraints::AddStretchBiasElementResidualAndHessian(const FSolverParticlesRange& Particles, const int32 ConstraintIndex, const int32 ConstraintIndexLocal, const FSolverReal Dt, TVec3<FSolverReal>& ParticleResidual, Chaos::PMatrix<FSolverReal, 3, 3>& ParticleHessian)
{
	const bool StiffnessWarpHasWeightMap = StiffnessWarp.HasWeightMap();
	const bool StiffnessWeftHasWeightMap = StiffnessWeft.HasWeightMap();
	const bool StiffnessBiasHasWeightMap = StiffnessBias.HasWeightMap();
	const bool DampingHasWeightMap = DampingRatio.HasWeightMap();
	const bool WarpScaleHasWeightMap = WarpScale.HasWeightMap();
	const bool WeftScaleHasWeightMap = WeftScale.HasWeightMap();

	const FSolverReal StiffnessWarpNoMap = (FSolverReal)StiffnessWarp;
	const FSolverReal StiffnessWeftNoMap = (FSolverReal)StiffnessWeft;
	const FSolverReal StiffnessBiasNoMap = (FSolverReal)StiffnessBias;
	const FSolverReal DampingNoMap = (FSolverReal)DampingRatio;
	const FSolverReal WarpScaleNoMap = (FSolverReal)WarpScale;
	const FSolverReal WeftScaleNoMap = (FSolverReal)WeftScale;
	const FSolverVec3 ExpStiffnessValue(StiffnessWeftNoMap, StiffnessWarpNoMap, StiffnessBiasNoMap);

	const FSolverMatrix22& DeltaUVInv = DeltaUVInverse[ConstraintIndex];

	const TVec3<int32>& Constraint = Constraints[ConstraintIndex];
	const int32 Idx0 = Constraint[0];
	const int32 Idx1 = Constraint[1];
	const int32 Idx2 = Constraint[2];
	if (Particles.InvM(Idx0) == (FSolverReal)0. && Particles.InvM(Idx1) == (FSolverReal)0. && Particles.InvM(Idx2) == (FSolverReal)0.)
	{
		return;
	}

	FSolverVec3 DXDu, DXDv;
	CalculateUVStretch(ConstraintIndex, Particles.P(Idx0), Particles.P(Idx1), Particles.P(Idx2), DXDu, DXDv);

	const FSolverReal DXDuLength = DXDu.Length();
	const FSolverReal DXDvLength = DXDv.Length();
	auto SafeRecip = [](const FSolverReal Len, const FSolverReal Fallback)
	{
		if (Len > UE_SMALL_NUMBER)
		{
			return 1.f / Len;
		}
		return Fallback;
	};
	const FSolverReal OneOverDXDuLen = SafeRecip(DXDuLength, 0.f);
	const FSolverReal OneOverDXDvLen = SafeRecip(DXDvLength, 0.f);
	const FSolverReal OneOverDXDuLenCube = OneOverDXDuLen * OneOverDXDuLen * OneOverDXDuLen;
	const FSolverReal OneOverDXDvLenCube = OneOverDXDvLen * OneOverDXDvLen * OneOverDXDvLen;

	const FSolverVec3 DXDuNormalized = DXDu * OneOverDXDuLen;
	const FSolverVec3 DXDvNormalized = DXDv * OneOverDXDvLen;

	const FSolverVec3 FinalStiffnesses = StiffnessScales[ConstraintIndex] * ExpStiffnessValue;

	// constraints
	const FSolverReal Cu = (DXDuLength - RestStretchLengths[ConstraintIndex][0] * WeftScaleNoMap); // stretch in weft direction
	const FSolverReal Cv = (DXDvLength - RestStretchLengths[ConstraintIndex][1] * WarpScaleNoMap); // stretch in warp direction
	const FSolverReal Cs = FSolverVec3::DotProduct(DXDuNormalized, DXDvNormalized) - RestStretchLengths[ConstraintIndex][2];

	//original uv constraint coefficients:
	const FSolverReal Coeffu =  Cu * OneOverDXDuLen * FinalStiffnesses[0] * Dt * Dt;
	const FSolverReal Coeffv =  Cv * OneOverDXDvLen * FinalStiffnesses[1] * Dt * Dt;
	const FSolverReal Coeffs =  Cs * FinalStiffnesses[2] * Dt * Dt;

	const FSolverVec3 GradCu1 = DXDuNormalized * DeltaUVInv.M[0];
	const FSolverVec3 GradCu2 = DXDuNormalized * DeltaUVInv.M[1];
	const FSolverVec3 GradCu0 = -GradCu1 - GradCu2;
	const FSolverVec3 GradCv1 = DXDvNormalized * DeltaUVInv.M[2];
	const FSolverVec3 GradCv2 = DXDvNormalized * DeltaUVInv.M[3];
	const FSolverVec3 GradCv0 = -GradCv1 - GradCv2;

	const FSolverReal StiffnessScaling = StiffnessPaddingRatio;

	const PMatrix<FSolverReal, 3, 2> DmRest = RestDmArray[ConstraintIndex];
	const FSolverMatrix22 DmMat = DmArray[ConstraintIndex], DmInvMat = DmInverse[ConstraintIndex];
	const PMatrix<FSolverReal, 3, 2> RestUVMatrix = DmRest * DeltaUVInv;
	const FSolverReal RestColumnInnProduct = RestUVMatrix.M[0] * RestUVMatrix.M[3] + RestUVMatrix.M[1] * RestUVMatrix.M[4] + RestUVMatrix.M[2] * RestUVMatrix.M[5];
	

	const PMatrix<FSolverReal, 2, 2> ActualUVMatrix = DmMat * DeltaUVInv;

	const FSolverReal ColumnInnProduct = ActualUVMatrix.M[0] * ActualUVMatrix.M[2] + ActualUVMatrix.M[1] * ActualUVMatrix.M[3];

	const FSolverVec2 TGSLu = DmMat.TransformPosition(FSolverVec2(DeltaUVInv.M[0], DeltaUVInv.M[1]));
	const FSolverVec2 TGSLv = DmMat.TransformPosition(FSolverVec2(DeltaUVInv.M[2], DeltaUVInv.M[3]));

	const FSolverReal TGSLuNormSquared = TGSLu.Length() * TGSLu.Length();
	const FSolverReal TGSLvNormSquared = TGSLv.Length() * TGSLv.Length();

	FSolverReal StiffnessPadding = FinalStiffnesses[0] + FinalStiffnesses[1] + FinalStiffnesses[2];
	StiffnessPadding *= StiffnessScaling;

	FSolverReal PaddingCoeff = 0.f;
	if (bDmInitialized)
	{
		PaddingCoeff = Dt* Dt* Measure[ConstraintIndex];
	}

	FSolverVec3 GradCs((FSolverReal)0.), GradCu((FSolverReal)0.), GradCv((FSolverReal)0.);
	//The following is the simplified version of the gradient:
	switch (ConstraintIndexLocal)
	{
		case 0:
			GradCs = -(DXDvNormalized * OneOverDXDuLen * DeltaUVInv.M[1] + DXDuNormalized * OneOverDXDvLen * DeltaUVInv.M[3]) - (DXDvNormalized * OneOverDXDuLen * DeltaUVInv.M[0] + DXDuNormalized * OneOverDXDvLen * DeltaUVInv.M[2]);
			GradCu = -DXDuNormalized * DeltaUVInv.M[0] - DXDuNormalized * DeltaUVInv.M[1];
			GradCv = -DXDvNormalized * DeltaUVInv.M[2] - DXDvNormalized * DeltaUVInv.M[3];
			break;
		case 1:
			GradCs = DXDvNormalized * OneOverDXDuLen * DeltaUVInv.M[0] + DXDuNormalized * OneOverDXDvLen * DeltaUVInv.M[2];
			GradCu = DXDuNormalized * DeltaUVInv.M[0];
			GradCv = DXDvNormalized * DeltaUVInv.M[2];
			break;
		case 2:
			GradCs = DXDvNormalized * OneOverDXDuLen * DeltaUVInv.M[1] + DXDuNormalized * OneOverDXDvLen * DeltaUVInv.M[3];
			GradCu = DXDuNormalized * DeltaUVInv.M[1];
			GradCv = DXDvNormalized * DeltaUVInv.M[3];
			break;
		default:
			break;
	}

	FSolverVec3 ResContribution((FSolverReal)0.);

	//residual part:
	if (ConstraintIndexLocal > 0)
	{
		for (int32 Alpha = 0; Alpha < 3; Alpha++)
		{
			//original uv constraints:
			ResContribution[Alpha] += Coeffu * DXDu[Alpha] * DeltaUVInv.M[ConstraintIndexLocal - 1];
			ResContribution[Alpha] += Coeffv * DXDv[Alpha] * DeltaUVInv.M[ConstraintIndexLocal + 1];

			//the new ones:
			ResContribution[Alpha] += Coeffs * GradCs[Alpha];
		}
	}
	else
	{
		for (int32 Alpha = 0; Alpha < 3; Alpha++)
		{
			ResContribution[Alpha] += Coeffs * GradCs[Alpha];
			//original uv constraint residuals:
			for (int32 Col = 0; Col < 2; Col++)
			{
				ResContribution[Alpha] -= Coeffu * DXDu[Alpha] * DeltaUVInv.M[Col];
				ResContribution[Alpha] -= Coeffv * DXDv[Alpha] * DeltaUVInv.M[Col + 2];
			}
		}
	}

	PMatrix<FSolverReal, 3, 3> HessianContribution((FSolverReal)0., (FSolverReal)0., (FSolverReal)0.);

	const FSolverReal RankOneuCoeff = FinalStiffnesses[0] * Dt * Dt * OneOverDXDuLen * OneOverDXDuLen;
	const FSolverReal RankOnevCoeff = FinalStiffnesses[1] * Dt * Dt * OneOverDXDvLen * OneOverDXDvLen;
	const FSolverReal HessianCoeffs = FinalStiffnesses[2] * Dt * Dt;
	const FSolverReal HessianCoeffu = FinalStiffnesses[0] * Dt * Dt;
	const FSolverReal HessianCoeffv = FinalStiffnesses[1] * Dt * Dt;

	//hessian part:
	if (ConstraintIndexLocal > 0)
	{
		for (int32 Alpha = 0; Alpha < 3; Alpha++)
		{
			for (int32 Beta = 0; Beta < 3; Beta++)
			{
				//constraint u:
				HessianContribution.SetAt(Alpha, Beta, HessianContribution.GetAt(Alpha, Beta) + RankOneuCoeff * DeltaUVInv.M[ConstraintIndexLocal - 1] * DeltaUVInv.M[ConstraintIndexLocal - 1] * DXDu[Alpha] * DXDu[Beta]);
				
				//constraint v:
				HessianContribution.SetAt(Alpha, Beta, HessianContribution.GetAt(Alpha, Beta) + RankOnevCoeff * DeltaUVInv.M[ConstraintIndexLocal + 1] * DeltaUVInv.M[ConstraintIndexLocal + 1] * DXDv[Alpha] * DXDv[Beta]);

			}
			//constraint s:
			HessianContribution.SetRow(Alpha, HessianContribution.GetRow(Alpha) + HessianCoeffs * GradCs[Alpha] * GradCs);
		}
		//code for stiffness padding:
		FSolverReal DmInvSum = 0.f;
		for (int32 Col = 0; Col < 2; Col++)
		{
			DmInvSum += DmInvMat.GetAt(ConstraintIndexLocal - 1, Col) * DmInvMat.GetAt(ConstraintIndexLocal - 1, Col);
		}
		for (int32 Alpha = 0; Alpha < 3; Alpha++)
		{
			HessianContribution.SetAt(Alpha, Alpha, HessianContribution.GetAt(Alpha, Alpha) + PaddingCoeff * StiffnessPadding * DmInvSum);
		}
	}
	else
	{
		for (int32 Alpha = 0; Alpha < 3; Alpha++)
		{
			for (int32 Beta = 0; Beta < 3; Beta++)
			{
				for (int32 Col = 0; Col < 2; Col++)
				{
					for (int32 Row = 0; Row < 2; Row++)
					{
						//constraint u (definitely could be simplified by adding u[Col] first):
						HessianContribution.SetAt(Alpha, Beta, HessianContribution.GetAt(Alpha, Beta) + RankOneuCoeff * DeltaUVInv.M[Col] * DeltaUVInv.M[Row] * DXDu[Alpha] * DXDu[Beta]);

						//constraint v (definitely could be simplified by adding v[Col] first):
						HessianContribution.SetAt(Alpha, Beta, HessianContribution.GetAt(Alpha, Beta) + RankOnevCoeff * DeltaUVInv.M[Col+2] * DeltaUVInv.M[Row+2] * DXDv[Alpha] * DXDv[Beta]);
					}
				}
			}
			HessianContribution.SetRow(Alpha, HessianContribution.GetRow(Alpha) + HessianCoeffs * GradCs[Alpha] * GradCs);
		}
		//code for stiffness padding:
		FSolverReal DmInvSum = 0.f;
		for (int32 Row = 0; Row < 2; Row++)
		{
			FSolverReal LocalDmSum = 0.f;
			for (int32 Col = 0; Col < 2; Col++)
			{
				LocalDmSum += DmInvMat.M[Row * 2 + Col];
			}
			DmInvSum += LocalDmSum * LocalDmSum;
		}
		for (int32 Alpha = 0; Alpha < 3; Alpha++)
		{
			HessianContribution.SetAt(Alpha, Alpha, HessianContribution.GetAt(Alpha, Alpha) + PaddingCoeff * StiffnessPadding * DmInvSum);
		}

	}

	ParticleResidual += ResContribution;
	ParticleHessian += HessianContribution;
}

void FXPBDStretchBiasElementConstraints::AddStretchBiasElementResidualAndHessian(const FSolverParticles& Particles, const int32 ConstraintIndex, const int32 ConstraintIndexLocal, const FSolverReal Dt, TVec3<FSolverReal>& ParticleResidual, Chaos::PMatrix<FSolverReal, 3, 3>& ParticleHessian)
{
	AddStretchBiasElementResidualAndHessian(FSolverParticlesRange(Particles), ConstraintIndex, ConstraintIndexLocal, Dt, ParticleResidual, ParticleHessian);
}


void FXPBDStretchBiasElementConstraints::AddInternalForceDifferential(const FSolverParticlesRange& InParticles, const TArray<TVector<FSolverReal, 3>>& DeltaParticles, TArray<TVector<FSolverReal, 3>>& Ndf)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(STAT_ChaosCGAnisoBiasElementAddInternalForceDifferential);
	const FSolverReal StiffnessWarpNoMap = (FSolverReal)StiffnessWarp;
	const FSolverReal StiffnessWeftNoMap = (FSolverReal)StiffnessWeft;
	const FSolverReal StiffnessBiasNoMap = (FSolverReal)StiffnessBias;
	const FSolverReal DampingNoMap = (FSolverReal)DampingRatio;
	const FSolverReal WarpScaleNoMap = (FSolverReal)WarpScale;
	const FSolverReal WeftScaleNoMap = (FSolverReal)WeftScale;
	const FSolverVec3 ExpStiffnessValue(StiffnessWeftNoMap, StiffnessWarpNoMap, StiffnessBiasNoMap);

	const int32 ParticleStart = 0;

	check(Ndf.Num() == InParticles.Size());

	const int32 ConstraintColorNum = ConstraintsPerColorStartIndex.Num() - 1;

	for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
	{
		const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
		const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
		PhysicsParallelFor(ColorSize, [&](const int32 Index)
			{
				const int32 ConstraintIndex = ColorStart + Index;
				const TVec3<int32>& Constraint = Constraints[ConstraintIndex];
				const int32 Idx0 = Constraint[0];
				const int32 Idx1 = Constraint[1];
				const int32 Idx2 = Constraint[2];

				if (InParticles.InvM(Idx0) == (FSolverReal)0. && InParticles.InvM(Idx1) == (FSolverReal)0. && InParticles.InvM(Idx2) == (FSolverReal)0.)
				{
					return;
				}

				FSolverVec3 DXDu, DXDv;
				CalculateUVStretch(ConstraintIndex, InParticles.P(Idx0), InParticles.P(Idx1), InParticles.P(Idx2), DXDu, DXDv);

				const FSolverReal DXDuLength = DXDu.Length();
				const FSolverReal DXDvLength = DXDv.Length();
				auto SafeRecip = [](const FSolverReal Len, const FSolverReal Fallback)
				{
					if (Len > UE_SMALL_NUMBER)
					{
						return 1.f / Len;
					}
					return Fallback;
				};
				const FSolverReal OneOverDXDuLen = SafeRecip(DXDuLength, 0.f);
				const FSolverReal OneOverDXDvLen = SafeRecip(DXDvLength, 0.f);

				const FSolverVec3 DXDuNormalized = DXDu * OneOverDXDuLen;
				const FSolverVec3 DXDvNormalized = DXDv * OneOverDXDvLen;

				// constraints
				const FSolverReal Cu = (DXDuLength - RestStretchLengths[ConstraintIndex][0] * WeftScaleNoMap); // stretch in weft direction
				const FSolverReal Cv = (DXDvLength - RestStretchLengths[ConstraintIndex][1] * WarpScaleNoMap); // stretch in warp direction
				const FSolverReal Cs = FSolverVec3::DotProduct(DXDuNormalized, DXDvNormalized) - RestStretchLengths[ConstraintIndex][2];


				const FSolverMatrix22& DeltaUVInv = DeltaUVInverse[ConstraintIndex];

				const FSolverVec3 GradCu1 = DXDuNormalized * DeltaUVInv.M[0];
				const FSolverVec3 GradCu2 = DXDuNormalized * DeltaUVInv.M[1];
				const FSolverVec3 GradCu0 = -GradCu1 - GradCu2;
				const FSolverVec3 GradCv1 = DXDvNormalized * DeltaUVInv.M[2];
				const FSolverVec3 GradCv2 = DXDvNormalized * DeltaUVInv.M[3];
				const FSolverVec3 GradCv0 = -GradCv1 - GradCv2;
				const FSolverVec3 GradCs1 = (DXDvNormalized * OneOverDXDuLen * DeltaUVInv.M[0] + DXDuNormalized * OneOverDXDvLen * DeltaUVInv.M[2]);
				const FSolverVec3 GradCs2 = (DXDvNormalized * OneOverDXDuLen * DeltaUVInv.M[1] + DXDuNormalized * OneOverDXDvLen * DeltaUVInv.M[3]);
				const FSolverVec3 GradCs0 = -GradCs1 - GradCs2;

				FSolverReal DeltaCu = 0.f, DeltaCv = 0.f, DeltaCs = 0.f;

				const FSolverVec3 DeltaP01 = DeltaParticles[Idx1 - ParticleStart] - DeltaParticles[Idx0 - ParticleStart];
				const FSolverVec3 DeltaP02 = DeltaParticles[Idx2 - ParticleStart] - DeltaParticles[Idx0 - ParticleStart];
				for (int32 Alpha = 0; Alpha < 3; Alpha++)
				{
					DeltaCu += DXDuNormalized[Alpha] * DeltaP01[Alpha] * DeltaUVInv.M[0];
					DeltaCu += DXDuNormalized[Alpha] * DeltaP02[Alpha] * DeltaUVInv.M[1];
					DeltaCv += DXDvNormalized[Alpha] * DeltaP01[Alpha] * DeltaUVInv.M[2];
					DeltaCv += DXDvNormalized[Alpha] * DeltaP02[Alpha] * DeltaUVInv.M[3];
					DeltaCs += GradCs1[Alpha] * DeltaP01[Alpha];
					DeltaCs += GradCs2[Alpha] * DeltaP02[Alpha];
				}

				const FSolverVec3 FinalStiffnesses = StiffnessScales[ConstraintIndex] * ExpStiffnessValue;

				FSolverVec3 DeltaDXDu, DeltaDXDv;
				CalculateUVStretch(ConstraintIndex, DeltaParticles[Idx0 - ParticleStart], DeltaParticles[Idx1 - ParticleStart], DeltaParticles[Idx2 - ParticleStart], DeltaDXDu, DeltaDXDv);
				const FSolverVec3 DeltaDXDuOverDxDuNorm = DeltaDXDu * OneOverDXDuLen;
				const FSolverVec3 DeltaDXDvOverDxDvNorm = DeltaDXDv * OneOverDXDvLen;
				FSolverVec3 DeltaGradCu0 = DeltaDXDuOverDxDuNorm * DeltaUVInv.M[0];
				FSolverVec3 DeltaGradCu1 = DeltaDXDuOverDxDuNorm * DeltaUVInv.M[1];
				FSolverVec3 DeltaGradCu2 = -DeltaGradCu0 - DeltaGradCu1;
				FSolverVec3 DeltaGradCv0 = DeltaDXDvOverDxDvNorm * DeltaUVInv.M[2];
				FSolverVec3 DeltaGradCv1 = DeltaDXDvOverDxDvNorm * DeltaUVInv.M[3];
				FSolverVec3 DeltaGradCv2 = -DeltaGradCv0 - DeltaGradCv1;


				Ndf[Idx0] += FinalStiffnesses[0] * DeltaCu * GradCu0 + FinalStiffnesses[0] * Cu * DeltaGradCu0;
				Ndf[Idx0] += FinalStiffnesses[1] * DeltaCv * GradCv0 + FinalStiffnesses[0] * Cu * DeltaGradCu1;
				Ndf[Idx0] += FinalStiffnesses[2] * DeltaCs * GradCs0 + FinalStiffnesses[0] * Cu * DeltaGradCu2;
				Ndf[Idx1] += FinalStiffnesses[0] * DeltaCu * GradCu1 + FinalStiffnesses[1] * Cv * DeltaGradCv0;
				Ndf[Idx1] += FinalStiffnesses[1] * DeltaCv * GradCv1 + FinalStiffnesses[1] * Cv * DeltaGradCv1;
				Ndf[Idx1] += FinalStiffnesses[2] * DeltaCs * GradCs1 + FinalStiffnesses[1] * Cv * DeltaGradCv2;
				Ndf[Idx2] += FinalStiffnesses[0] * DeltaCu * GradCu2;
				Ndf[Idx2] += FinalStiffnesses[1] * DeltaCv * GradCv2;
				Ndf[Idx2] += FinalStiffnesses[2] * DeltaCs * GradCs2;
			});
	}
}

void FXPBDStretchBiasElementConstraints::AddInternalForceDifferential(const FSolverParticles& InParticles, const TArray<TVector<FSolverReal, 3>>& DeltaParticles, TArray<TVector<FSolverReal, 3>>& Ndf)
{
	AddInternalForceDifferential(FSolverParticlesRange(InParticles), DeltaParticles, Ndf);
}


}  // End namespace Chaos::Softs
