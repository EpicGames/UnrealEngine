// Copyright Epic Games, Inc. All Rights Reserved.

#define EXPLICIT_VECTOR4 1

#include "Math/Vector.isph"
#include "Chaos/PBDSofts.isph"

static inline varying float SafeDivide(const varying float Numerator, const varying float Denominator)
{
	return select(Denominator <= FLOAT_SMALL_NUMBER, (uniform float)0, Numerator / Denominator);
}

static void CalcConstraintsAndGrads(const FVector3f& P0, const FVector3f& P1, const FVector3f& P2, const FVector4f& DeltaUVInv,
	const FVector3f& RestStretchLengths, const float WarpScale, const float WeftScale,
	float& Cu, float& Cv, float& Cs, 
	FVector3f& GradCu1, FVector3f& GradCu2, FVector3f& GradCv1, FVector3f& GradCv2, FVector3f& GradCs1, FVector3f& GradCs2)
{
	const FVector3f P01 = P1 - P0;
	const FVector3f P02 = P2 - P0;
	const FVector3f DXDu = DeltaUVInv.V[0] * P01 + DeltaUVInv.V[1] * P02;
	const FVector3f DXDv = DeltaUVInv.V[2] * P01 + DeltaUVInv.V[3] * P02;

	const float DXDuLength = VectorSize(DXDu);
	const float DXDvLength = VectorSize(DXDv);

	const float OneOverDXDuLen = SafeDivide(1.f, DXDuLength);
	const float OneOverDXDvLen = SafeDivide(1.f, DXDvLength);

	const FVector3f DXDuNormalized = OneOverDXDuLen * DXDu;
	const FVector3f DXDvNormalized = OneOverDXDvLen * DXDv;

	Cu = DXDuLength - RestStretchLengths.V[0] * WeftScale;
	Cv = DXDvLength - RestStretchLengths.V[1] * WarpScale;
	Cs = VectorDot(DXDuNormalized, DXDvNormalized) - RestStretchLengths.V[2];

	GradCu1 = DXDuNormalized * DeltaUVInv.V[0];
	GradCu2 = DXDuNormalized * DeltaUVInv.V[1];
	GradCv1 = DXDvNormalized * DeltaUVInv.V[2];
	GradCv2 = DXDvNormalized * DeltaUVInv.V[3];
	GradCs1 = (DXDvNormalized * OneOverDXDuLen * DeltaUVInv.V[0] + DXDuNormalized * OneOverDXDvLen * DeltaUVInv.V[2]);
	GradCs2 = (DXDvNormalized * OneOverDXDuLen * DeltaUVInv.V[1] + DXDuNormalized * OneOverDXDvLen * DeltaUVInv.V[3]);
}

static inline float CalcDLambda(const float StiffnessValue, const float Dt, const float MinStiffness, const float Lambda,
	const float InvM0, const float InvM1, const float InvM2,
	const float C, const FVector3f& GradC1, const FVector3f& GradC2)
{
	const float Alpha = 1.f / (StiffnessValue  * Dt * Dt);
	const float Denom = InvM0 * VectorSizeSquared(GradC1 + GradC2) + InvM1 * VectorSizeSquared(GradC1) + InvM2 * VectorSizeSquared(GradC2) + Alpha;
	const float DLambda = (C - Alpha * Lambda) / Denom;
	return select(StiffnessValue < MinStiffness, (uniform float)0, DLambda);
}

static inline float CalcDamping(const float StiffnessValue, const float CombinedInvMass, const float DampingRatio)
{
	return 2.f * DampingRatio * sqrt(StiffnessValue / CombinedInvMass);
}

static inline float CalcDLambda(const float StiffnessValue, const float Dt, const float MinStiffness, const float Lambda, const float Damping,
	const float InvM0, const float InvM1, const float InvM2,
	const FVector3f& V0TimesDt, const FVector3f& V1TimesDt, const FVector3f& V2TimesDt, 
	const float C, const FVector3f& GradC1, const FVector3f& GradC2)
{
	const float Alpha = 1.f / (StiffnessValue  * Dt * Dt);
	const float Gamma = Alpha * Damping * Dt;
	const float DampingTerm = Gamma * (VectorDot(V1TimesDt, GradC1) + VectorDot(V2TimesDt, GradC2) - VectorDot(V0TimesDt, GradC1 + GradC2));
	const float Denom = (1. + Gamma) * (InvM0 * VectorSizeSquared(GradC1 + GradC2) + InvM1 * VectorSizeSquared(GradC1) + InvM2 * VectorSizeSquared(GradC2)) + Alpha;
	const float DLambda = (C - Alpha * Lambda + DampingTerm) / Denom;
	return select(StiffnessValue < MinStiffness, (uniform float)0, DLambda);
}

export void ApplyXPBDStretchBiasConstraints(uniform FVector4f PandInvM[],
									const uniform FIntVector Constraints[],
									const uniform FVector3f RestStretchLengths[],
									const uniform FVector4f DeltaUVInverses[],
									const uniform FVector3f StiffnessScales[],
									uniform FVector3f Lambdas[],
									const uniform float Dt,
									const uniform float MinStiffness,
									const uniform FVector3f& Stiffness,
									const uniform float WarpScale,
									const uniform float WeftScale,
									const uniform int32 NumConstraints)
{

	foreach(i = 0 ... NumConstraints)
	{
		const varying FIntVector Constraint = VectorLoad(&Constraints[extract(i,0)]);

		const varying int32 i0 = Constraint.V[0];
		const varying int32 i1 = Constraint.V[1];
		const varying int32 i2 = Constraint.V[2];
		
		const varying FVector4f PandInvM0 = VectorGather(&PandInvM[i0]);
		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[i1]);
		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[i2]);

		varying FVector3f P0, P1, P2;
		varying float M0, M1, M2;
		
		UnzipPandInvM(PandInvM0, P0, M0);
		UnzipPandInvM(PandInvM1, P1, M1);
		UnzipPandInvM(PandInvM2, P2, M2);

		const varying FVector4f DeltaUVInverse = VectorLoad(&DeltaUVInverses[extract(i,0)]);
		const varying FVector3f RestStretchLength = VectorLoad(&RestStretchLengths[extract(i,0)]);

		varying float Cu, Cv, Cs;
		varying FVector3f GradCu1, GradCu2, GradCv1, GradCv2, GradCs1, GradCs2;
		CalcConstraintsAndGrads(P0, P1, P2, DeltaUVInverse, RestStretchLength, WarpScale, WeftScale, Cu, Cv, Cs,
			GradCu1, GradCu2, GradCv1, GradCv2, GradCs1, GradCs2);
		
		FVector3f Lambda = VectorLoad(&Lambdas[extract(i,0)]);
		const varying FVector3f StiffnessScale = VectorLoad(&StiffnessScales[extract(i,0)]);
		const varying FVector3f FinalStiffness = Stiffness * StiffnessScale;

		const varying float DLambdaU = CalcDLambda(FinalStiffness.V[0], Dt, MinStiffness, Lambda.V[0], M0, M1, M2, Cu, GradCu1, GradCu2);
		const varying float DLambdaV = CalcDLambda(FinalStiffness.V[1], Dt, MinStiffness, Lambda.V[1], M0, M1, M2, Cv, GradCv1, GradCv2);
		const varying float DLambdaS = CalcDLambda(FinalStiffness.V[2], Dt, MinStiffness, Lambda.V[2], M0, M1, M2, Cs, GradCs1, GradCs2);
		const varying FVector3f DLambda = {{DLambdaU, DLambdaV, DLambdaS}};

		Lambda = Lambda + DLambda;
		VectorStore(&Lambdas[extract(i,0)], Lambda);
		
		const varying FVector3f DP1 = DLambdaU * GradCu1 + DLambdaV * GradCv1 + DLambdaS * GradCs1;
		const varying FVector3f DP2 = DLambdaU * GradCu2 + DLambdaV * GradCv2 + DLambdaS * GradCs2;

		if (M0 > 0)
		{
			VectorScatter(&PandInvM[i0], SetVector4(P0 + M0 * (DP1 + DP2), M0));
		}
		if (M1 > 0)
		{
			VectorScatter(&PandInvM[i1], SetVector4(P1 - M1 * DP1, M1));
		}
		if (M2 > 0)
		{
			VectorScatter(&PandInvM[i2], SetVector4(P2 - M2 * DP2, M2));
		}
	}
}

export void ApplyXPBDStretchBiasConstraintsWithDamping(uniform FVector4f PandInvM[],
									const uniform FVector3f X[],
									const uniform FIntVector Constraints[],
									const uniform FVector3f RestStretchLengths[],
									const uniform FVector4f DeltaUVInverses[],
									const uniform FVector3f StiffnessScales[],
									uniform FVector3f Lambdas[],
									const uniform float Dt,
									const uniform float MinStiffness,
									const uniform FVector3f& Stiffness,
									const uniform float DampingRatio,
									const uniform float WarpScale,
									const uniform float WeftScale,
									const uniform int32 NumConstraints)
{

	foreach(i = 0 ... NumConstraints)
	{
		const varying FIntVector Constraint = VectorLoad(&Constraints[extract(i,0)]);

		const varying int32 i0 = Constraint.V[0];
		const varying int32 i1 = Constraint.V[1];
		const varying int32 i2 = Constraint.V[2];
		
		const varying FVector4f PandInvM0 = VectorGather(&PandInvM[i0]);
		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[i1]);
		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[i2]);

		varying FVector3f P0, P1, P2;
		varying float M0, M1, M2;
		
		UnzipPandInvM(PandInvM0, P0, M0);
		UnzipPandInvM(PandInvM1, P1, M1);
		UnzipPandInvM(PandInvM2, P2, M2);

		const varying FVector4f DeltaUVInverse = VectorLoad(&DeltaUVInverses[extract(i,0)]);
		const varying FVector3f RestStretchLength = VectorLoad(&RestStretchLengths[extract(i,0)]);

		varying float Cu, Cv, Cs;
		varying FVector3f GradCu1, GradCu2, GradCv1, GradCv2, GradCs1, GradCs2;
		CalcConstraintsAndGrads(P0, P1, P2, DeltaUVInverse, RestStretchLength, WarpScale, WeftScale, Cu, Cv, Cs,
			GradCu1, GradCu2, GradCv1, GradCv2, GradCs1, GradCs2);
		
		const varying FVector3f X0 = VectorGather(&X[i0]);
		const varying FVector3f X1 = VectorGather(&X[i1]);
		const varying FVector3f X2 = VectorGather(&X[i2]);
		const varying FVector3f V0TimesDt = P0 - X0;
		const varying FVector3f V1TimesDt = P1 - X1;
		const varying FVector3f V2TimesDt = P2 - X2;
		const varying float CombinedInvMass = M0 + M1 + M2;
		
		FVector3f Lambda = VectorLoad(&Lambdas[extract(i,0)]);
		const varying FVector3f StiffnessScale = VectorLoad(&StiffnessScales[extract(i,0)]);
		const varying FVector3f FinalStiffness = Stiffness * StiffnessScale;
		const varying float DLambdaU = CalcDLambda(FinalStiffness.V[0], Dt, MinStiffness, Lambda.V[0], 
			CalcDamping(FinalStiffness.V[0], CombinedInvMass, DampingRatio),
			M0, M1, M2, V0TimesDt, V1TimesDt, V2TimesDt, Cu, GradCu1, GradCu2);
		const varying float DLambdaV = CalcDLambda(FinalStiffness.V[1], Dt, MinStiffness, Lambda.V[1], 
			CalcDamping(FinalStiffness.V[1], CombinedInvMass, DampingRatio),
			M0, M1, M2, V0TimesDt, V1TimesDt, V2TimesDt, Cv, GradCv1, GradCv2);
		const varying float DLambdaS = CalcDLambda(FinalStiffness.V[2], Dt, MinStiffness, Lambda.V[2], 
			CalcDamping(FinalStiffness.V[2], CombinedInvMass, DampingRatio),
			M0, M1, M2, V0TimesDt, V1TimesDt, V2TimesDt, Cs, GradCs1, GradCs2);
		const varying FVector3f DLambda = {{DLambdaU, DLambdaV, DLambdaS}};

		Lambda = Lambda + DLambda;
		VectorStore(&Lambdas[extract(i,0)], Lambda);
		
		const varying FVector3f DP1 = DLambdaU * GradCu1 + DLambdaV * GradCv1 + DLambdaS * GradCs1;
		const varying FVector3f DP2 = DLambdaU * GradCu2 + DLambdaV * GradCv2 + DLambdaS * GradCs2;

		if (M0 > 0)
		{
			VectorScatter(&PandInvM[i0], SetVector4(P0 + M0 * (DP1 + DP2), M0));
		}
		if (M1 > 0)
		{
			VectorScatter(&PandInvM[i1], SetVector4(P1 - M1 * DP1, M1));
		}
		if (M2 > 0)
		{
			VectorScatter(&PandInvM[i2], SetVector4(P2 - M2 * DP2, M2));
		}
	}
}

export void ApplyXPBDStretchBiasConstraintsWithMaps(uniform FVector4f PandInvM[],
											const uniform FIntVector Constraints[],
											const uniform FVector3f RestStretchLengths[],
											const uniform FVector4f DeltaUVInverses[],
											const uniform FVector3f StiffnessScales[],
											uniform FVector3f Lambdas[],
											const uniform float Dt,
											const uniform float MinStiffness,
											const uniform bool StiffnessWarpHasMap,
											const uniform uint8 StiffnessWarpIndices[],
											const uniform float StiffnessWarpTable[],
											const uniform bool StiffnessWeftHasMap,
											const uniform uint8 StiffnessWeftIndices[],
											const uniform float StiffnessWeftTable[],
											const uniform bool StiffnessBiasHasMap,
											const uniform uint8 StiffnessBiasIndices[],
											const uniform float StiffnessBiasTable[],
											const uniform bool WarpScaleHasMap,
											const uniform uint8 WarpScaleIndices[],
											const uniform float WarpScaleTable[],
											const uniform bool WeftScaleHasMap,
											const uniform uint8 WeftScaleIndices[],
											const uniform float WeftScaleTable[],
											const uniform int32 NumConstraints)
{

	foreach(i = 0 ... NumConstraints)
	{
		const varying FIntVector Constraint = VectorLoad(&Constraints[extract(i,0)]);

		const varying int32 i0 = Constraint.V[0];
		const varying int32 i1 = Constraint.V[1];
		const varying int32 i2 = Constraint.V[2];
		const varying int8 StiffnessWarpIndex = StiffnessWarpHasMap ? StiffnessWarpIndices[i] : 0;
		const varying int8 StiffnessWeftIndex = StiffnessWeftHasMap ? StiffnessWeftIndices[i] : 0;
		const varying int8 StiffnessBiasIndex = StiffnessBiasHasMap ? StiffnessBiasIndices[i] : 0;
		const varying int8 WarpScaleIndex = WarpScaleHasMap ? WarpScaleIndices[i] : 0;
		const varying int8 WeftScaleIndex = WeftScaleHasMap ? WeftScaleIndices[i] : 0;
		
		const varying FVector4f PandInvM0 = VectorGather(&PandInvM[i0]);
		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[i1]);
		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[i2]);

		varying FVector3f P0, P1, P2;
		varying float M0, M1, M2;
		
		UnzipPandInvM(PandInvM0, P0, M0);
		UnzipPandInvM(PandInvM1, P1, M1);
		UnzipPandInvM(PandInvM2, P2, M2);

		const varying FVector4f DeltaUVInverse = VectorLoad(&DeltaUVInverses[extract(i,0)]);
		const varying FVector3f RestStretchLength = VectorLoad(&RestStretchLengths[extract(i,0)]);
		
		#pragma ignore warning(perf)
		const varying float WarpScale = WarpScaleTable[WarpScaleIndex];
		#pragma ignore warning(perf)
		const varying float WeftScale = WeftScaleTable[WeftScaleIndex];

		varying float Cu, Cv, Cs;
		varying FVector3f GradCu1, GradCu2, GradCv1, GradCv2, GradCs1, GradCs2;
		CalcConstraintsAndGrads(P0, P1, P2, DeltaUVInverse, RestStretchLength, WarpScale, WeftScale, Cu, Cv, Cs,
			GradCu1, GradCu2, GradCv1, GradCv2, GradCs1, GradCs2);		
		
		#pragma ignore warning(perf)
		const varying float StiffnessWarp = StiffnessWarpTable[StiffnessWarpIndex];
		#pragma ignore warning(perf)
		const varying float StiffnessWeft = StiffnessWeftTable[StiffnessWeftIndex];
		#pragma ignore warning(perf)
		const varying float StiffnessBias = StiffnessBiasTable[StiffnessBiasIndex];
		const varying FVector3f Stiffness = {{StiffnessWeft, StiffnessWarp, StiffnessBias}};

		FVector3f Lambda = VectorLoad(&Lambdas[extract(i,0)]);
		const varying FVector3f StiffnessScale = VectorLoad(&StiffnessScales[extract(i,0)]);
		const varying FVector3f FinalStiffness = Stiffness * StiffnessScale;

		const varying float DLambdaU = CalcDLambda(FinalStiffness.V[0], Dt, MinStiffness, Lambda.V[0], M0, M1, M2, Cu, GradCu1, GradCu2);
		const varying float DLambdaV = CalcDLambda(FinalStiffness.V[1], Dt, MinStiffness, Lambda.V[1], M0, M1, M2, Cv, GradCv1, GradCv2);
		const varying float DLambdaS = CalcDLambda(FinalStiffness.V[2], Dt, MinStiffness, Lambda.V[2], M0, M1, M2, Cs, GradCs1, GradCs2);
		const varying FVector3f DLambda = {{DLambdaU, DLambdaV, DLambdaS}};

		Lambda = Lambda + DLambda;
		VectorStore(&Lambdas[extract(i,0)], Lambda);
		
		const varying FVector3f DP1 = DLambdaU * GradCu1 + DLambdaV * GradCv1 + DLambdaS * GradCs1;
		const varying FVector3f DP2 = DLambdaU * GradCu2 + DLambdaV * GradCv2 + DLambdaS * GradCs2;

		if (M0 > 0)
		{
			VectorScatter(&PandInvM[i0], SetVector4(P0 + M0 * (DP1 + DP2), M0));
		}
		if (M1 > 0)
		{
			VectorScatter(&PandInvM[i1], SetVector4(P1 - M1 * DP1, M1));
		}
		if (M2 > 0)
		{
			VectorScatter(&PandInvM[i2], SetVector4(P2 - M2 * DP2, M2));
		}
	}
}

export void ApplyXPBDStretchBiasConstraintsWithDampingAndMaps(uniform FVector4f PandInvM[],
											const uniform FVector3f X[],
											const uniform FIntVector Constraints[],
											const uniform FVector3f RestStretchLengths[],
											const uniform FVector4f DeltaUVInverses[],
											const uniform FVector3f StiffnessScales[],
											uniform FVector3f Lambdas[],
											const uniform float Dt,
											const uniform float MinStiffness,
											const uniform bool StiffnessWarpHasMap,
											const uniform uint8 StiffnessWarpIndices[],
											const uniform float StiffnessWarpTable[],
											const uniform bool StiffnessWeftHasMap,
											const uniform uint8 StiffnessWeftIndices[],
											const uniform float StiffnessWeftTable[],
											const uniform bool StiffnessBiasHasMap,
											const uniform uint8 StiffnessBiasIndices[],
											const uniform float StiffnessBiasTable[],
											const uniform bool DampingHasMap,
											const uniform uint8 DampingIndices[],
											const uniform float DampingTable[],
											const uniform bool WarpScaleHasMap,
											const uniform uint8 WarpScaleIndices[],
											const uniform float WarpScaleTable[],
											const uniform bool WeftScaleHasMap,
											const uniform uint8 WeftScaleIndices[],
											const uniform float WeftScaleTable[],
											const uniform int32 NumConstraints)
{

	foreach(i = 0 ... NumConstraints)
	{
		const varying FIntVector Constraint = VectorLoad(&Constraints[extract(i,0)]);

		const varying int32 i0 = Constraint.V[0];
		const varying int32 i1 = Constraint.V[1];
		const varying int32 i2 = Constraint.V[2];
		const varying int8 StiffnessWarpIndex = StiffnessWarpHasMap ? StiffnessWarpIndices[i] : 0;
		const varying int8 StiffnessWeftIndex = StiffnessWeftHasMap ? StiffnessWeftIndices[i] : 0;
		const varying int8 StiffnessBiasIndex = StiffnessBiasHasMap ? StiffnessBiasIndices[i] : 0;
		const varying int8 WarpScaleIndex = WarpScaleHasMap ? WarpScaleIndices[i] : 0;
		const varying int8 WeftScaleIndex = WeftScaleHasMap ? WeftScaleIndices[i] : 0;
		const varying int8 DampingIndex = DampingHasMap ? DampingIndices[i] : 0;
		
		const varying FVector4f PandInvM0 = VectorGather(&PandInvM[i0]);
		const varying FVector4f PandInvM1 = VectorGather(&PandInvM[i1]);
		const varying FVector4f PandInvM2 = VectorGather(&PandInvM[i2]);

		varying FVector3f P0, P1, P2;
		varying float M0, M1, M2;
		
		UnzipPandInvM(PandInvM0, P0, M0);
		UnzipPandInvM(PandInvM1, P1, M1);
		UnzipPandInvM(PandInvM2, P2, M2);

		const varying FVector4f DeltaUVInverse = VectorLoad(&DeltaUVInverses[extract(i,0)]);
		const varying FVector3f RestStretchLength = VectorLoad(&RestStretchLengths[extract(i,0)]);		
		
		#pragma ignore warning(perf)
		const varying float WarpScale = WarpScaleTable[WarpScaleIndex];
		#pragma ignore warning(perf)
		const varying float WeftScale = WeftScaleTable[WeftScaleIndex];

		varying float Cu, Cv, Cs;
		varying FVector3f GradCu1, GradCu2, GradCv1, GradCv2, GradCs1, GradCs2;
		CalcConstraintsAndGrads(P0, P1, P2, DeltaUVInverse, RestStretchLength, WarpScale, WeftScale, Cu, Cv, Cs,
			GradCu1, GradCu2, GradCv1, GradCv2, GradCs1, GradCs2);
		
		const varying FVector3f X0 = VectorGather(&X[i0]);
		const varying FVector3f X1 = VectorGather(&X[i1]);
		const varying FVector3f X2 = VectorGather(&X[i2]);
		const varying FVector3f V0TimesDt = P0 - X0;
		const varying FVector3f V1TimesDt = P1 - X1;
		const varying FVector3f V2TimesDt = P2 - X2;
		const varying float CombinedInvMass = M0 + M1 + M2;
		
		#pragma ignore warning(perf)
		const varying float StiffnessWarp = StiffnessWarpTable[StiffnessWarpIndex];
		#pragma ignore warning(perf)
		const varying float StiffnessWeft = StiffnessWeftTable[StiffnessWeftIndex];
		#pragma ignore warning(perf)
		const varying float StiffnessBias = StiffnessBiasTable[StiffnessBiasIndex];
		const varying FVector3f Stiffness = {{StiffnessWeft, StiffnessWarp, StiffnessBias}};

		#pragma ignore warning(perf)
		const varying float DampingRatio = DampingTable[DampingIndex];
		
		FVector3f Lambda = VectorLoad(&Lambdas[extract(i,0)]);
		const varying FVector3f StiffnessScale = VectorLoad(&StiffnessScales[extract(i,0)]);
		const varying FVector3f FinalStiffness = Stiffness * StiffnessScale;
		const varying float DLambdaU = CalcDLambda(FinalStiffness.V[0], Dt, MinStiffness, Lambda.V[0], 
			CalcDamping(FinalStiffness.V[0], CombinedInvMass, DampingRatio),
			M0, M1, M2, V0TimesDt, V1TimesDt, V2TimesDt, Cu, GradCu1, GradCu2);
		const varying float DLambdaV = CalcDLambda(FinalStiffness.V[1], Dt, MinStiffness, Lambda.V[1], 
			CalcDamping(FinalStiffness.V[1], CombinedInvMass, DampingRatio),
			M0, M1, M2, V0TimesDt, V1TimesDt, V2TimesDt, Cv, GradCv1, GradCv2);
		const varying float DLambdaS = CalcDLambda(FinalStiffness.V[2], Dt, MinStiffness, Lambda.V[2], 
			CalcDamping(FinalStiffness.V[2], CombinedInvMass, DampingRatio),
			M0, M1, M2, V0TimesDt, V1TimesDt, V2TimesDt, Cs, GradCs1, GradCs2);
		const varying FVector3f DLambda = {{DLambdaU, DLambdaV, DLambdaS}};

		Lambda = Lambda + DLambda;
		VectorStore(&Lambdas[extract(i,0)], Lambda);
		
		const varying FVector3f DP1 = DLambdaU * GradCu1 + DLambdaV * GradCv1 + DLambdaS * GradCs1;
		const varying FVector3f DP2 = DLambdaU * GradCu2 + DLambdaV * GradCv2 + DLambdaS * GradCs2;

		if (M0 > 0)
		{
			VectorScatter(&PandInvM[i0], SetVector4(P0 + M0 * (DP1 + DP2), M0));
		}
		if (M1 > 0)
		{
			VectorScatter(&PandInvM[i1], SetVector4(P1 - M1 * DP1, M1));
		}
		if (M2 > 0)
		{
			VectorScatter(&PandInvM[i2], SetVector4(P2 - M2 * DP2, M2));
		}
	}
}