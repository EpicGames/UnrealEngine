// Copyright Epic Games, Inc. All Rights Reserved.
#include "ChaosDebugDraw/ChaosDDContext.h"
#include "ChaosDebugDraw/ChaosDDLog.h"
#include "ChaosDebugDraw/ChaosDDFrame.h"
#include "ChaosDebugDraw/ChaosDDTimeline.h"
#include "ChaosLog.h"
#include "HAL/IConsoleManager.h"


DEFINE_LOG_CATEGORY(LogChaosDD);

#if CHAOS_DEBUG_DRAW

UE_DEFINE_THREAD_SINGLETON_TLS(ChaosDD::Private::FChaosDDContext, CHAOS_API)

namespace ChaosDD::Private
{
	bool bChaosDebugDraw_EnableGlobalQueue = true;
	FAutoConsoleVariableRef CVarChaos_DebugDraw_EnableGlobalQueue(TEXT("p.Chaos.DebugDraw.EnableGlobalQueue"), bChaosDebugDraw_EnableGlobalQueue, TEXT(""));

	bool FChaosDDContext::bDebugDrawEnabled = false;

	FCriticalSection FChaosDDContext::GlobalFrameCS;
	FChaosDDFramePtr FChaosDDContext::GlobalFrame;
	int32 FChaosDDContext::GlobalCommandBudget = 20000;

	//
	//
	// Timeline Context
	//
	//

	void FChaosDDTimelineContext::BeginFrame(const FChaosDDTimelinePtr& InTimeline, double InTime, double InDt)
	{
		check(!bInContext);

		if (FChaosDDContext::IsDebugDrawEnabled())
		{
			FChaosDDContext& Context = FChaosDDContext::Get();
			PreviousFrame = Context.Frame;

			if (InTimeline.IsValid())
			{
				Timeline = InTimeline;
				Timeline->BeginFrame(InTime, InDt);
				Context.Frame = Timeline->GetActiveFrame();
			}
			else
			{
				Context.Frame.Reset();
			}

			bInContext = true;
		}
	}

	void FChaosDDTimelineContext::EndFrame()
	{
		if (bInContext)
		{
			if (Timeline.IsValid())
			{
				Timeline->EndFrame();
				Timeline.Reset();
			}

			FChaosDDContext& Context = FChaosDDContext::Get();
			Context.Frame = PreviousFrame;
			PreviousFrame.Reset();

			bInContext = false;
		}
	}

	FChaosDDScopeTimelineContext::FChaosDDScopeTimelineContext(const FChaosDDTimelinePtr& InTimeline, double InTime, double InDt)
	{
		Context.BeginFrame(InTimeline, InTime, InDt);
	}

	FChaosDDScopeTimelineContext::~FChaosDDScopeTimelineContext()
	{
		Context.EndFrame();
	}

	//
	//
	// Task Context
	//
	//

	FChaosDDTaskParentContext::FChaosDDTaskParentContext()
		: Frame(FChaosDDContext::Get().Frame)
	{
	}

	void FChaosDDTaskContext::BeginThread(const FChaosDDTaskParentContext& InParentDDContext)
	{
		check(!bInContext);

		// NOTE: (UE-216178) We used to pass a reference to the parent FChaosDDContext directly to the
		// child thread and pulled the FramePointer from it in BeginThread. That is not safe because 
		// the parent thread may also be helping with tasks and so the Frame on that context will be 
		// getting set/unset. Instead we copy the Frame pointer on the parent thread and pass it in.
		if (FChaosDDContext::IsDebugDrawEnabled())
		{
			FChaosDDContext& Context = FChaosDDContext::Get();
			PreviousFrame = Context.Frame;
			Context.Frame = InParentDDContext.Frame;
			bInContext = true;
		}
	}

	void FChaosDDTaskContext::EndThread()
	{
		if (bInContext)
		{
			FChaosDDContext& Context = FChaosDDContext::Get();
			Context.Frame = PreviousFrame;
			PreviousFrame.Reset();
			bInContext = false;
		}
	}

	FChaosDDScopeTaskContext::FChaosDDScopeTaskContext(const FChaosDDTaskParentContext& InParentDDContext)
	{
		Context.BeginThread(InParentDDContext);
	}

	FChaosDDScopeTaskContext::~FChaosDDScopeTaskContext()
	{
		Context.EndThread();
	}

	//
	//
	// Thread Local Context
	//
	//

	FChaosDDContext::FChaosDDContext()
	{
	}

	const FChaosDDFramePtr& FChaosDDContext::GetGlobalFrame()
	{
		FScopeLock Lock(&GlobalFrameCS);

		if (!GlobalFrame.IsValid())
		{
			CreateGlobalFrame();
		}

		return GlobalFrame;
	}

	void FChaosDDContext::CreateGlobalFrame()
	{
		if (bChaosDebugDraw_EnableGlobalQueue)
		{
			FScopeLock Lock(&GlobalFrameCS);

			GlobalFrame = MakeShared<FChaosDDGlobalFrame>(GlobalCommandBudget);
		}
	}

	FChaosDDFramePtr FChaosDDContext::ExtractGlobalFrame()
	{
		FScopeLock Lock(&GlobalFrameCS);

		// Handle toggling the cvar in the runtime
		if (!bChaosDebugDraw_EnableGlobalQueue && GlobalFrame.IsValid())
		{
			GlobalFrame.Reset();
		}

		if (GlobalFrame.IsValid())
		{
			return GlobalFrame->ExtractFrame();
		}

		return {};
	}

	void FChaosDDContext::SetGlobalDrawRegion(const FSphere3d& InDrawRegion)
	{
		FScopeLock Lock(&GlobalFrameCS);

		if (GlobalFrame.IsValid())
		{
			GlobalFrame->SetDrawRegion(InDrawRegion);
		}
	}

	void FChaosDDContext::SetGlobalCommandBudget(int32 InCommandBudget)
	{
		FScopeLock Lock(&GlobalFrameCS);

		GlobalCommandBudget = InCommandBudget;

		if (GlobalFrame.IsValid())
		{
			GlobalFrame->SetCommandBudget(InCommandBudget);
		}
	}
}

#endif