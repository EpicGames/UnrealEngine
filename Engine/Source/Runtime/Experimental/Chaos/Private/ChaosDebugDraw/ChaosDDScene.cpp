// Copyright Epic Games, Inc. All Rights Reserved.
#include "ChaosDebugDraw/ChaosDDScene.h"
#include "ChaosDebugDraw/ChaosDDContext.h"
#include "ChaosDebugDraw/ChaosDDFrame.h"
#include "ChaosDebugDraw/ChaosDDLog.h"
#include "ChaosDebugDraw/ChaosDDRenderer.h"
#include "ChaosDebugDraw/ChaosDDTimeline.h"

#if CHAOS_DEBUG_DRAW

namespace ChaosDD::Private
{
	template<typename LambdaType>
	void VisitTimelines(const TArray<FChaosDDTimelineWeakPtr>& Timelines, const LambdaType& Visitor)
	{
		for (const FChaosDDTimelineWeakPtr& TimelineWeak : Timelines)
		{
			if (const FChaosDDTimelinePtr& Timeline = TimelineWeak.Pin())
			{
				Visitor(Timeline);
			}
		}
	}

	FChaosDDScene::FChaosDDScene(const FString& InName, bool bInIsServer)
		: Name(InName)
		, DrawRegion(FVector::Zero(), 0.0)
		, CommandBudget(20000)
		, bIsServer(bInIsServer)
		, bRenderEnabled(true)
	{
	}

	FChaosDDScene::~FChaosDDScene()
	{
	}

	bool FChaosDDScene::IsServer() const
	{
		return bIsServer;
	}

	void FChaosDDScene::SetRenderEnabled(bool bInRenderEnabled)
	{
		FScopeLock Lock(&TimelinesCS);

		bRenderEnabled = bInRenderEnabled;
	}

	bool FChaosDDScene::IsRenderEnabled() const
	{
		FScopeLock Lock(&TimelinesCS);

		return bRenderEnabled;
	}

	void FChaosDDScene::SetDrawRegion(const FSphere3d& InDrawRegion)
	{
		FScopeLock Lock(&TimelinesCS);

		DrawRegion = InDrawRegion;

		VisitTimelines(Timelines,
			[&InDrawRegion](const FChaosDDTimelinePtr& Timeline)
			{
				Timeline->SetDrawRegion(InDrawRegion);
			});

		// The global timeline doesn't use a draw region because it could be shared
		// among multiple viewports and they probably don't want the same region
		//FChaosDDContext::SetGlobalDrawRegion(InDrawRegion);
	}

	const FSphere3d& FChaosDDScene::GetDrawRegion() const
	{
		FScopeLock Lock(&TimelinesCS);

		return DrawRegion;
	}

	void FChaosDDScene::SetCommandBudget(int32 InCommandBudget)
	{
		FScopeLock Lock(&TimelinesCS);

		CommandBudget = InCommandBudget;

		VisitTimelines(Timelines,
			[InCommandBudget](const FChaosDDTimelinePtr& Timeline)
			{
				Timeline->SetCommandBudget(InCommandBudget);
			});

		FChaosDDContext::SetGlobalCommandBudget(InCommandBudget);
	}

	int32 FChaosDDScene::GetCommandBudget() const
	{
		FScopeLock Lock(&TimelinesCS);

		return CommandBudget;
	}

	FChaosDDTimelinePtr FChaosDDScene::CreateTimeline(const FString& InName)
	{
		FScopeLock Lock(&TimelinesCS);

		FChaosDDTimelinePtr Timeline = MakeShared<FChaosDDTimeline>(InName, CommandBudget);

		Timelines.Add(Timeline.ToWeakPtr());

		return Timeline;
	}

	TArray<FChaosDDFramePtr> FChaosDDScene::GetLatestFrames()
	{
		return GetFrames();
	}

	TArray<FChaosDDFramePtr> FChaosDDScene::GetFrames()
	{
		FScopeLock Lock(&TimelinesCS);

		TArray<FChaosDDFramePtr> Frames;

		VisitTimelines(Timelines,
			[&Frames](const FChaosDDTimelinePtr& Timeline)
			{
				Timeline->GetFrames(Frames);
			});

		return Frames;
	}

	void FChaosDDScene::PruneTimelines()
	{
		FScopeLock Lock(&TimelinesCS);

		// Remove all timelines that have been deleted
		Timelines.RemoveAll(
			[](const FChaosDDTimelineWeakPtr& Timeline)
			{
				return !Timeline.IsValid();
			});
	}
}

#endif
