// Copyright Epic Games, Inc. All Rights Reserved.

#if UE_TRACE_ENABLED

#include "ChaosInsights/ChaosInsightsMacros.h"

#include "CoreMinimal.h"
#include "Trace/Trace.h"
#include "Trace/Trace.inl"
#include "Misc/CString.h"
#include "HAL/PlatformTime.h"

UE_TRACE_CHANNEL(ChaosLocksChannel, "Chaos physics read/write lock acquisition events (begin, acquired, end) with cycle timestamps. Useful for diagnosing lock contention in the physics solver.")

UE_TRACE_EVENT_BEGIN(Chaos, LockAcquireBegin)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(bool, bIsWrite)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Chaos, LockAcquired)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(Chaos, LockAcquireEnd)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
UE_TRACE_EVENT_END()

namespace Chaos::Insights
{
	uint64 BeginLockAcquireEvent(ELockEventType Type)
	{
		uint64 Id = FPlatformTime::Cycles64();
		UE_TRACE_LOG(Chaos, LockAcquireBegin, ChaosLocksChannel)
			<< LockAcquireBegin.Cycle(FPlatformTime::Cycles64())
			<< LockAcquireBegin.bIsWrite(Type == ELockEventType::RWLockWriteLock || Type == ELockEventType::Mutex);

		return Id;
	}

	void AcquiredLock()
	{
		UE_TRACE_LOG(Chaos, LockAcquired, ChaosLocksChannel)
			<< LockAcquired.Cycle(FPlatformTime::Cycles64());
	}

	void EndLockAcquireEvent()
	{
		UE_TRACE_LOG(Chaos, LockAcquireEnd, ChaosLocksChannel)
			<< LockAcquireEnd.Cycle(FPlatformTime::Cycles64());
	}
}

#endif // UE_TRACE_ENABLED
