// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosLog.h"

#include "ChaosModule.h"
#include "CoreMinimal.h"
#include "ProfilingDebugging/CsvProfiler.h"

DEFINE_LOG_CATEGORY(LogChaosGeneral);
DEFINE_LOG_CATEGORY(LogChaosThread);
DEFINE_LOG_CATEGORY(LogChaosSimulation);
DEFINE_LOG_CATEGORY(LogChaosDebug);
DEFINE_LOG_CATEGORY(LogChaos);
DEFINE_LOG_CATEGORY(LogChaosDataflow);

// UE_DEPRECATED(5.7, "Use LogChaos instead")
PRAGMA_DISABLE_DEPRECATION_WARNINGS
DEFINE_LOG_CATEGORY(UManagedArrayLogging);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

CSV_DEFINE_CATEGORY_MODULE(CHAOS_API, Chaos, true);
CSV_DEFINE_CATEGORY_MODULE(CHAOS_API, PhysicsVerbose, false);
CSV_DEFINE_CATEGORY_MODULE(CHAOS_API, PhysicsCounters, false);