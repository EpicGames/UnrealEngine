// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosModule.h"
#include "CoreMinimal.h"
#include "ChaosLog.h"
#include "ChaosVisualDebugger/ChaosVisualDebuggerTrace.h"
#include "Modules/ModuleManager.h"
#include "Misc/CommandLine.h"

IMPLEMENT_MODULE(FChaosEngineModule, Chaos);

namespace Chaos
{
	extern CHAOS_API int GSingleThreadedPhysics;
	extern CHAOS_API bool bSingleWorkerPhysics;
	extern CHAOS_API int32 MinNumberOfWorkerThreadToRunOnSingleWorker;
}

void FChaosEngineModule::StartupModule()
{
	if (FModuleManager::Get().ModuleExists(TEXT("GeometryCore")))
	{
		FModuleManager::Get().LoadModule("GeometryCore");
	}

	if(FParse::Param(FCommandLine::Get(), TEXT("SingleThreadedPhysics")))
	{
		Chaos::GSingleThreadedPhysics = 1;
		Chaos::bSingleWorkerPhysics = true;
	}

#if CHAOS_VISUAL_DEBUGGER_ENABLED
	FChaosVisualDebuggerTrace::RegisterEventHandlers();
#endif
}

void FChaosEngineModule::ShutdownModule()
{
#if CHAOS_VISUAL_DEBUGGER_ENABLED
	FChaosVisualDebuggerTrace::UnregisterEventHandlers();
#endif
}
