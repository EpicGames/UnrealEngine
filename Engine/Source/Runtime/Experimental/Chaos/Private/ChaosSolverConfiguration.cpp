// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosSolverConfiguration.h"
#include "Chaos/PBDRigidsEvolutionGBF.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosSolverConfiguration)

FChaosSolverConfiguration::FChaosSolverConfiguration()
	: PositionIterations(Chaos::FPBDRigidsEvolutionGBF::DefaultNumPositionIterations)
	, VelocityIterations(Chaos::FPBDRigidsEvolutionGBF::DefaultNumVelocityIterations)
	, ProjectionIterations(Chaos::FPBDRigidsEvolutionGBF::DefaultNumProjectionIterations)
	, CollisionMarginFraction(Chaos::FPBDRigidsEvolutionGBF::DefaultCollisionMarginFraction)
	, CollisionMarginMax(Chaos::FPBDRigidsEvolutionGBF::DefaultCollisionMarginMax)
	, CollisionCullDistance(Chaos::FPBDRigidsEvolutionGBF::DefaultCollisionCullDistance)
	, CollisionMaxPushOutVelocity(Chaos::FPBDRigidsEvolutionGBF::DefaultCollisionMaxPushOutVelocity)
	, CollisionInitialOverlapDepenetrationVelocity(Chaos::FPBDRigidsEvolutionGBF::DefaultCollisionDepenetrationVelocity)
	, ClusterConnectionFactor(1.0f)
	, ClusterUnionConnectionType(EClusterUnionMethod::DelaunayTriangulation)
	, bGenerateCollisionData(false)
	, bGenerateBreakData(false)
	, bGenerateTrailingData(false)
	, Iterations_DEPRECATED(Chaos::FPBDRigidsEvolutionGBF::DefaultNumPositionIterations)
	, PushOutIterations_DEPRECATED(Chaos::FPBDRigidsEvolutionGBF::DefaultNumVelocityIterations)
	, bGenerateContactGraph_DEPRECATED(true)
{
}

void FChaosSolverConfiguration::MoveRenamedPropertyValues()
{
	if (Iterations_DEPRECATED != Chaos::FPBDRigidsEvolutionGBF::DefaultNumPositionIterations)
	{
		PositionIterations = Iterations_DEPRECATED;
	}
	if (PushOutIterations_DEPRECATED != Chaos::FPBDRigidsEvolutionGBF::DefaultNumVelocityIterations)
	{
		VelocityIterations = PushOutIterations_DEPRECATED;
	}
}

