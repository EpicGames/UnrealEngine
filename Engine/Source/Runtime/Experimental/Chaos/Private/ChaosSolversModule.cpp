// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosSolversModule.h"
#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"

#include "ChaosStats.h"
#include "ChaosLog.h"
#include "HAL/PlatformProcess.h"
#include "Misc/CoreDelegates.h"
#include "HAL/IConsoleManager.h"
#include "PhysicsSolver.h"
#include "FramePro/FramePro.h"
#include "Chaos/BoundingVolume.h"
#include "Chaos/PBDRigidParticles.h"
#include "Chaos/UniformGrid.h"
#include "UObject/Class.h"
#include "Misc/App.h"
#include "Chaos/PhysicalMaterials.h"
#include "Chaos/ChaosDebugNameDefines.h"

TAutoConsoleVariable<int32> CVarChaosThreadEnabled(
	TEXT("p.Chaos.DedicatedThreadEnabled"),
	1,
	TEXT("Enables a dedicated physics task/thread for Chaos tasks.")
	TEXT("0: Disabled")
	TEXT("1: Enabled"));

TAutoConsoleVariable<Chaos::FRealSingle> CVarDedicatedThreadDesiredHz(
	TEXT("p.Chaos.Thread.DesiredHz"),
	60.0f,
	TEXT("Desired update rate of the dedicated physics thread in Hz/FPS (Default 60.0f)"));

TAutoConsoleVariable<int32> CVarDedicatedThreadSyncThreshold(
	TEXT("p.Chaos.Thread.WaitThreshold"),
	0,
	TEXT("Desired wait time in ms before the game thread stops waiting to sync physics and just takes the last result. (default 16ms)")
);

namespace Chaos
{
#if !UE_BUILD_SHIPPING
	CHAOS_API extern bool bPendingHierarchyDump;
#endif

	FInternalDefaultSettings GDefaultChaosSettings;
}

FSolverStateStorage::FSolverStateStorage()
	: Solver(nullptr)
{

}

FChaosSolversModule* FChaosSolversModule::GetModule()
{
	static FChaosSolversModule* Instance = nullptr;

	if(!Instance)
	{
		Instance = new FChaosSolversModule();
		Instance->StartupModule();	//todo:remove the module api
	}

	return Instance;
}

FChaosSolversModule::FChaosSolversModule()
	: SolverActorClassProvider(nullptr)
	, SettingsProvider(nullptr)
	, bPersistentTaskSpawned(false)
	, PhysicsAsyncTask(nullptr)
	, PhysicsInnerTask(nullptr)
	, SolverActorClass(nullptr)
	, SolverActorRequiredBaseClass(nullptr)
#if STATS
	, AverageUpdateTime(0.0f)
	, TotalAverageUpdateTime(0.0f)
	, Fps(0.0f)
	, EffectiveFps(0.0f)
#endif
#if WITH_EDITOR
	, bPauseSolvers(false)
	, SingleStepCounter(0)
#endif
	, bModuleInitialized(false)
{
#if WITH_EDITOR
	if(!(IsRunningDedicatedServer() || IsRunningGame()))
	{
		// In the editor we begin with everything paused so we don't needlessly tick
		// the physics solvers until PIE begins. Delegates are bound in FPhysScene_ChaosPauseHandler
		// to handle editor world transitions. In games and -game we want to just let them tick
		bPauseSolvers = true;
	}
#endif
}

void FChaosSolversModule::StartupModule()
{
	Initialize();
}

void FChaosSolversModule::ShutdownModule()
{
	Shutdown();

	FCoreDelegates::OnPreExit.RemoveAll(this);
}

void FChaosSolversModule::Initialize()
{
	if(!bModuleInitialized)
	{
		// Bind to the material manager
		Chaos::FPhysicalMaterialManager& MaterialManager = Chaos::FPhysicalMaterialManager::Get();
		OnCreateMaterialHandle = MaterialManager.OnMaterialCreated.Add(Chaos::FMaterialCreatedDelegate::CreateRaw(this, &FChaosSolversModule::OnCreateMaterial));
		OnDestroyMaterialHandle = MaterialManager.OnMaterialDestroyed.Add(Chaos::FMaterialDestroyedDelegate::CreateRaw(this, &FChaosSolversModule::OnDestroyMaterial));
		OnUpdateMaterialHandle = MaterialManager.OnMaterialUpdated.Add(Chaos::FMaterialUpdatedDelegate::CreateRaw(this, &FChaosSolversModule::OnUpdateMaterial));

		OnCreateMaterialMaskHandle = MaterialManager.OnMaterialMaskCreated.Add(Chaos::FMaterialMaskCreatedDelegate::CreateRaw(this, &FChaosSolversModule::OnCreateMaterialMask));
		OnDestroyMaterialMaskHandle = MaterialManager.OnMaterialMaskDestroyed.Add(Chaos::FMaterialMaskDestroyedDelegate::CreateRaw(this, &FChaosSolversModule::OnDestroyMaterialMask));
		OnUpdateMaterialMaskHandle = MaterialManager.OnMaterialMaskUpdated.Add(Chaos::FMaterialMaskUpdatedDelegate::CreateRaw(this, &FChaosSolversModule::OnUpdateMaterialMask));

		bModuleInitialized = true;
	}
}

void FChaosSolversModule::Shutdown()
{
	using namespace Chaos;
	if(bModuleInitialized)
	{
		for(FPhysicsSolverBase* Solver : AllSolvers)
		{
			Solver->WaitOnPendingTasks_External();
		}

		// Unbind material events
		FPhysicalMaterialManager& MaterialManager = FPhysicalMaterialManager::Get();
		MaterialManager.OnMaterialCreated.Remove(OnCreateMaterialHandle);
		MaterialManager.OnMaterialDestroyed.Remove(OnDestroyMaterialHandle);
		MaterialManager.OnMaterialUpdated.Remove(OnUpdateMaterialHandle);

		bModuleInitialized = false;
	}
}

bool FChaosSolversModule::IsPersistentTaskEnabled() const
{
	return CVarChaosThreadEnabled.GetValueOnGameThread() == 1;
}

bool FChaosSolversModule::IsPersistentTaskRunning() const
{
	return bPersistentTaskSpawned;
}

Chaos::FPersistentPhysicsTask* FChaosSolversModule::GetDedicatedTask() const
{
	return PhysicsInnerTask;
}

void FChaosSolversModule::SyncTask(bool bForceBlockingSync /*= false*/)
{
#if 0
	// Hard lock the physics thread before syncing our data
	FChaosScopedPhysicsThreadLock ScopeLock(bForceBlockingSync ? MAX_uint32 : (uint32)(CVarDedicatedThreadSyncThreshold.GetValueOnGameThread()));

	// This will either get the results because physics finished, or fall back on whatever physics last gave us
	// to allow the game thread to continue on without stalling.
	PhysicsInnerTask->SyncProxiesFromCache(ScopeLock.DidGetLock());

	// Update stats if necessary
	UpdateStats();
#endif
}

Chaos::FPBDRigidsSolver* FChaosSolversModule::CreateSolver(UObject* InOwner, Chaos::FReal InAsyncDt, Chaos::EThreadingMode InThreadingMode, const FName& DebugName)
{
	LLM_SCOPE(ELLMTag::Chaos);
	using namespace Chaos;

	FChaosScopeSolverLock SolverScopeLock;
	
	EMultiBufferMode SolverBufferMode = InThreadingMode == EThreadingMode::SingleThread ? EMultiBufferMode::Single : EMultiBufferMode::Double;
	FPBDRigidsSolver* NewSolver = new FPBDRigidsSolver(SolverBufferMode, InOwner, InAsyncDt);
	AllSolvers.Add(NewSolver);

	// Add The solver to the owner list
	TArray<FPhysicsSolverBase*>& OwnerSolverList = SolverMap.FindOrAdd(InOwner);
	OwnerSolverList.Add(NewSolver);

#if CHAOS_SOLVER_DEBUG_NAME
    // Add solver number to solver name
	const FName NewDebugName = *FString::Printf(TEXT("%s (%d)"), DebugName == NAME_None ? TEXT("Solver") : *DebugName.ToString(), AllSolvers.Num() - 1);
	NewSolver->SetDebugName(NewDebugName);
#endif

	// Set up the material lists on the new solver, copying from the current primary list
	{
		FPhysicalMaterialManager& Manager =	Chaos::FPhysicalMaterialManager::Get();
		FPhysicsSceneGuardScopedWrite ScopedWrite(NewSolver->GetExternalDataLock_External());
		NewSolver->QueryMaterials_External = Manager.GetPrimaryMaterials_External();
		NewSolver->QueryMaterialMasks_External = Manager.GetPrimaryMaterialMasks_External();
		NewSolver->SimMaterials = Manager.GetPrimaryMaterials_External();
		NewSolver->SimMaterialMasks = Manager.GetPrimaryMaterialMasks_External();
	}

	return NewSolver;
}

void FChaosSolversModule::MigrateSolver(Chaos::FPhysicsSolverBase* InSolver, const UObject* InNewOwner)
{
	checkSlow(IsInGameThread());

	if(InSolver->GetOwner() == InNewOwner)
	{
		// No migration
		return;
	}

	if(const UObject* CurrentOwner = InSolver->GetOwner())
	{
		TArray<Chaos::FPhysicsSolverBase*>* OwnerList = SolverMap.Find(CurrentOwner);
		ensure(OwnerList->Remove(InSolver));
	}

	InSolver->SetOwner(InNewOwner);

	if(InNewOwner)
	{
		TArray<Chaos::FPhysicsSolverBase*>& OwnerList = SolverMap.FindOrAdd(InNewOwner);
		OwnerList.Add(InSolver);
	}
}

UClass* FChaosSolversModule::GetSolverActorClass() const
{
	check(SolverActorClassProvider);
	return SolverActorClassProvider->GetSolverActorClass();
}

bool FChaosSolversModule::IsValidSolverActorClass(UClass* Class) const
{
	return Class->IsChildOf(SolverActorRequiredBaseClass);
}

void FChaosSolversModule::DestroySolver(Chaos::FPhysicsSolverBase* InSolver)
{
	LLM_SCOPE(ELLMTag::Chaos);

	FChaosScopeSolverLock SolverScopeLock;

	if(AllSolvers.Remove(InSolver) > 0)
	{
		//should this be a find ref check?
		if(TArray<Chaos::FPhysicsSolverBase*>* OwnerList = SolverMap.Find(InSolver->GetOwner()))
		{
			ensureMsgf(OwnerList->Remove(InSolver), TEXT("Removed a solver from the global list but not an owner list."));
		}
	}
	else if(InSolver)
	{
		UE_LOGF(LogChaosGeneral, Warning, "Passed valid solver state to DestroySolverState but it wasn't in the solver storage list! Make sure it was created using the Chaos module.");
	}

	if(InSolver)
	{
		Chaos::FPhysicsSolverBase::DestroySolver(*InSolver);
	}
}

const TArray<Chaos::FPhysicsSolverBase*>& FChaosSolversModule::GetAllSolvers() const
{
	return AllSolvers;
}

TArray<const Chaos::FPhysicsSolverBase*> FChaosSolversModule::GetSolvers(const UObject* InOwner) const
{
	// Can't just return the ptr from TMap::Find here as it's likely these solver lists will be sent to
	// the physics thread. In which case the solver pointers are stable but the container pointers are not
	TArray<const Chaos::FPhysicsSolverBase*> Solvers;
	GetSolvers(InOwner, Solvers);
	return Solvers;
}

TArray<Chaos::FPhysicsSolverBase*> FChaosSolversModule::GetSolversMutable(const UObject* InOwner)
{
	// Can't just return the ptr from TMap::Find here as it's likely these solver lists will be sent to
	// the physics thread. In which case the solver pointers are stable but the container pointers are not
	TArray<Chaos::FPhysicsSolverBase*> Solvers;
	GetSolversMutable(InOwner, Solvers);
	return Solvers;
}

void FChaosSolversModule::GetSolvers(const UObject* InOwner, TArray<const Chaos::FPhysicsSolverBase*>& OutSolvers) const
{
	if(const TArray<Chaos::FPhysicsSolverBase*>* Solvers = SolverMap.Find(InOwner))
	{
		OutSolvers.Append(*Solvers);
	}
}

void FChaosSolversModule::GetSolversMutable(const UObject* InOwner, TArray<Chaos::FPhysicsSolverBase*>& OutSolvers)
{
	if(TArray<Chaos::FPhysicsSolverBase*>* Solvers = SolverMap.Find(InOwner))
	{
		OutSolvers.Append(*Solvers);
	}
}

TAutoConsoleVariable<FString> DumpHier_ElementBuckets(
	TEXT("p.Chaos.DumpHierElementBuckets"),
	TEXT("1,4,8,16,32,64,128,256,512"),
	TEXT("Distribution buckets for dump hierarchy stats command"));

void FChaosSolversModule::DumpHierarchyStats(int32* OutOptMaxCellElements)
{
	TArray<FString> BucketStrings;
	DumpHier_ElementBuckets.GetValueOnGameThread().ParseIntoArray(BucketStrings, TEXT(","));
	
	// 2 extra for the 0 bucket at the start and the larger bucket at the end
	const int32 NumBuckets = BucketStrings.Num() + 2;
	TArray<int32> BucketSizes;
	BucketSizes.AddZeroed(NumBuckets);
	BucketSizes.Last() = MAX_int32;

	for(int32 BucketIndex = 1; BucketIndex < NumBuckets - 1; ++BucketIndex)
	{
		BucketSizes[BucketIndex] = FCString::Atoi(*BucketStrings[BucketIndex - 1]);
	}
	BucketSizes.Sort();

	TArray<int32> BucketCounts;
	BucketCounts.AddZeroed(NumBuckets);

	const int32 NumSolvers = AllSolvers.Num();
	for(int32 SolverIndex = 0; SolverIndex < NumSolvers; ++SolverIndex)
	{
		Chaos::FPhysicsSolverBase* Solver = AllSolvers[SolverIndex];
#if TODO_REIMPLEMENT_SPATIAL_ACCELERATION_ACCESS
		if(const Chaos::ISpatialAcceleration<Chaos::FReal,3>* SpatialAcceleration = Solver->GetSpatialAcceleration())
		{
#if !UE_BUILD_SHIPPING
			SpatialAcceleration->DumpStats();
#endif
			Solver->ReleaseSpatialAcceleration();
		}
#endif
#if 0

		const TArray<Chaos::FAABB3>& Boxes = Hierarchy->GetWorldSpaceBoxes();

		if(Boxes.Num() > 0)
		{
			FString OutputString = "\n\n";
			OutputString += FString::Printf(TEXT("Solver %d - Hierarchy Stats\n"));

			const Chaos::TUniformGrid<Chaos::FReal, 3>& Grid = Hierarchy->GetGrid();

			const int32 NumCells = Grid.GetNumCells();
			const FVector Min = Grid.MinCorner();
			const FVector Max = Grid.MaxCorner();
			const FVector Extent = Max - Min;

			OutputString += FString::Printf(TEXT("Grid:\n\tCells: [%d, %d, %d] (%d)\n\tMin: %s\n\tMax: %s\n\tExtent: %s\n"),
				Grid.Counts()[0],
				Grid.Counts()[1],
				Grid.Counts()[2],
				NumCells,
				*Min.ToString(),
				*Max.ToString(),
				*Extent.ToString()
				);

			int32 CellsL0 = 0;
			int32 TotalElems = 0;
			int32 MaxElements = 0;
			const int32 NumHeirElems = Hierarchy->GetElements().Num();
			for(int32 ElemIndex = 0; ElemIndex < NumHeirElems; ++ElemIndex)
			{
				const TArray<int32>& CellElems = Hierarchy->GetElements()[ElemIndex];

				const int32 NumCellEntries = CellElems.Num();

				if(NumCellEntries > 0)
				{
					++CellsL0;
				}

				if(NumCellEntries > MaxElements)
				{
					MaxElements = NumCellEntries;
				}

				TotalElems += NumCellEntries;

				for(int32 BucketIndex = 1; BucketIndex < NumBuckets; ++BucketIndex)
				{
					if(NumCellEntries >= BucketSizes[BucketIndex - 1] && NumCellEntries < BucketSizes[BucketIndex])
					{
						BucketCounts[BucketIndex]++;
						break;
					}
				}
			}

			if(OutOptMaxCellElements)
			{
				(*OutOptMaxCellElements) = MaxElements;
			}

			const Chaos::FReal AveragePopulatedCount = (Chaos::FReal)TotalElems / (Chaos::FReal)CellsL0;

			OutputString += FString::Printf(TEXT("\n\tL0: %d\n\tAvg elements per populated cell: %.5f\n\tTotal elems: %d"),
				CellsL0,
				AveragePopulatedCount,
				TotalElems);

			int32 MaxBucketCount = 0;
			for(int32 Count : BucketCounts)
			{
				if(Count > MaxBucketCount)
				{
					MaxBucketCount = Count;
				}
			}

			const int32 MaxChars = 20;
			const Chaos::FReal CountPerCharacter = (Chaos::FReal)MaxBucketCount / (Chaos::FReal)MaxChars;

			OutputString += TEXT("\n\nElement Count Distribution:\n");

			for(int32 BucketIndex = 1; BucketIndex < NumBuckets; ++BucketIndex)
			{
				const int32 NumChars = (Chaos::FReal)BucketCounts[BucketIndex] / (Chaos::FReal)CountPerCharacter;

				if(BucketIndex < (NumBuckets - 1))
				{
					OutputString += FString::Printf(TEXT("\t[%4d - %4d) (%4d) |"), BucketSizes[BucketIndex - 1], BucketSizes[BucketIndex], BucketCounts[BucketIndex]);
				}
				else
				{
					OutputString += FString::Printf(TEXT("\t[%4d -  inf) (%4d) |"), BucketSizes[BucketIndex - 1], BucketCounts[BucketIndex]);
				}

				for(int32 CharIndex = 0; CharIndex < NumChars; ++CharIndex)
				{
					OutputString += TEXT("-");
				}

				OutputString += TEXT("\n");
			}

			OutputString += TEXT("\n--------------------------------------------------");
			
			UE_LOGF(LogChaos, Warning, "%ls", *OutputString);
		}
#endif

#if TODO_REIMPLEMENT_SPATIAL_ACCELERATION_ACCESS
		Solver->ReleaseSpatialAcceleration();
#endif

#if !UE_BUILD_SHIPPING
		Chaos::bPendingHierarchyDump = true;	//mark solver pending dump to get more info
#endif
	}
}

DECLARE_CYCLE_STAT(TEXT("PhysicsDedicatedStats"), STAT_PhysicsDedicatedStats, STATGROUP_ChaosDedicated);	//this is a hack, needed to make stat group turn on
DECLARE_FLOAT_COUNTER_STAT(TEXT("PhysicsThreadTotalTime(ms)"), STAT_PhysicsThreadTotalTime, STATGROUP_ChaosDedicated);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumActiveConstraints"), STAT_NumActiveConstraintsDedicated, STATGROUP_ChaosDedicated);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumActiveParticles"), STAT_NumActiveParticlesDedicated, STATGROUP_ChaosDedicated);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumActiveCollisionPoints"), STAT_NumActiveCollisionPointsDedicated, STATGROUP_ChaosDedicated);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumActiveShapes"), STAT_NumActiveShapesDedicated, STATGROUP_ChaosDedicated);

#if 0
void FChaosSolversModule::UpdateStats()
{
#if STATS
	SCOPE_CYCLE_COUNTER(STAT_PhysicsStatUpdate);
	SCOPE_CYCLE_COUNTER(STAT_PhysicsDedicatedStats);

	Chaos::FPersistentPhysicsTaskStatistics PhysStats = PhysicsInnerTask->GetNextThreadStatistics_GameThread();

	if(PhysStats.NumUpdates > 0)
	{
		AverageUpdateTime = PhysStats.AccumulatedTime / (FReal)PhysStats.NumUpdates;
		TotalAverageUpdateTime = PhysStats.ActualAccumulatedTime / (FReal)PhysStats.NumUpdates;
		Fps = 1.0f / AverageUpdateTime;
		EffectiveFps = 1.0f / TotalAverageUpdateTime;
	}

	// Only set the stats if something is actually running
	if(Fps != 0.0f)
	{
		SET_FLOAT_STAT(STAT_PhysicsThreadTime, AverageUpdateTime * 1000.0f);
		SET_FLOAT_STAT(STAT_PhysicsThreadTimeEff, TotalAverageUpdateTime * 1000.0f);
		SET_FLOAT_STAT(STAT_PhysicsThreadFps, Fps);
		SET_FLOAT_STAT(STAT_PhysicsThreadFpsEff, EffectiveFps);

		if (Fps != 0.0f && PhysStats.SolverStats.Num() > 0)
		{
			PerSolverStats = PhysStats.AccumulateSolverStats();
		}

		SET_FLOAT_STAT(STAT_PhysicsThreadTotalTime, AverageUpdateTime * 1000.0f);
		SET_DWORD_STAT(STAT_NumActiveConstraintsDedicated, PerSolverStats.NumActiveConstraints);
		SET_DWORD_STAT(STAT_NumActiveParticlesDedicated, PerSolverStats.NumActiveParticles);
		SET_DWORD_STAT(STAT_NumActiveCollisionPointsDedicated, PerSolverStats.EvolutionStats.ActiveCollisionPoints);
		SET_DWORD_STAT(STAT_NumActiveShapesDedicated, PerSolverStats.EvolutionStats.ActiveShapes);

	}

#if FRAMEPRO_ENABLED

	// Custom framepro stats for graphs
	const Chaos::FRealSingle AvgUpdateMs = AverageUpdateTime * 1000.f;
	const Chaos::FRealSingle AvgEffectiveUpdateMs = TotalAverageUpdateTime * 1000.0f;

	FRAMEPRO_CUSTOM_STAT("Chaos_Thread_Fps", Fps, "ChaosThread", "FPS", FRAMEPRO_COLOUR(255,255,255));
	FRAMEPRO_CUSTOM_STAT("Chaos_Thread_EffectiveFps", EffectiveFps, "ChaosThread", "FPS", FRAMEPRO_COLOUR(255,255,255));
	FRAMEPRO_CUSTOM_STAT("Chaos_Thread_Time", AvgUpdateMs, "ChaosThread", "ms", FRAMEPRO_COLOUR(255,255,255));
	FRAMEPRO_CUSTOM_STAT("Chaos_Thread_EffectiveTime", AvgEffectiveUpdateMs, "ChaosThread", "ms", FRAMEPRO_COLOUR(255,255,255));

	FRAMEPRO_CUSTOM_STAT("Chaos_Thread_NumActiveParticles", PerSolverStats.NumActiveParticles, "ChaosThread", "Particles", FRAMEPRO_COLOUR(255,255,255));
	FRAMEPRO_CUSTOM_STAT("Chaos_Thread_NumConstraints", PerSolverStats.NumActiveConstraints, "ChaosThread", "Constraints", FRAMEPRO_COLOUR(255,255,255));
	FRAMEPRO_CUSTOM_STAT("Chaos_Thread_NumAllocatedParticles", PerSolverStats.NumAllocatedParticles, "ChaosThread", "Particles", FRAMEPRO_COLOUR(255,255,255));
	FRAMEPRO_CUSTOM_STAT("Chaos_Thread_NumPaticleIslands", PerSolverStats.NumParticleIslands, "ChaosThread", "Islands", FRAMEPRO_COLOUR(255,255,255));

	const int32 NumSolvers = AllSolvers.Num();
#if 0
	for(int32 SolverIndex = 0; SolverIndex < NumSolvers; ++SolverIndex)
	{
		Chaos::PBDRigidsSolver* Solver = AllSolvers[SolverIndex];

		const Chaos::TBoundingVolume<Chaos::FPBDRigidParticles>* Hierarchy = Solver->GetSpatialAcceleration();

		if(Hierarchy)
		{
			FRAMEPRO_CUSTOM_STAT("Chaos_Thread_Hierarchy_NumObjects", Hierarchy->GlobalObjects().Num(), "ChaosThread", "Objects");
		}
		Solver->ReleaseSpatialAcceleration();
	}
#endif

#endif 

#endif
}
#endif

#if WITH_EDITOR
void FChaosSolversModule::PauseSolvers()
{
	bPauseSolvers = true;
	UE_LOGF(LogChaosDebug, Verbose, "Pausing solvers.");
	// Sync physics to allow last minute updates
	if(IsPersistentTaskRunning())
	{
		SyncTask(true);
	}
}

void FChaosSolversModule::ResumeSolvers()
{
	bPauseSolvers = false;
	UE_LOGF(LogChaosDebug, Verbose, "Resuming solvers.");
}

void FChaosSolversModule::SingleStepSolvers()
{
	bPauseSolvers = true;
	SingleStepCounter.Increment();
	UE_LOGF(LogChaosDebug, Verbose, "Single-stepping solvers.");
	// Sync physics to allow last minute updates
	if(IsPersistentTaskRunning())
	{
		SyncTask(true);
	}
}

bool FChaosSolversModule::ShouldStepSolver(int32& InOutSingleStepCounter) const
{
	const int32 counter = SingleStepCounter.GetValue();
	const bool bShouldStepSolver = !(bPauseSolvers && InOutSingleStepCounter == counter);
	InOutSingleStepCounter = counter;
	return bShouldStepSolver;
}
#endif  // #if WITH_EDITOR

void FChaosSolversModule::OnUpdateMaterial(Chaos::FMaterialHandle InHandle)
{

	// Grab the material
	Chaos::FChaosPhysicsMaterial* Material = InHandle.Get();

	if(ensure(Material))
	{
		for(Chaos::FPhysicsSolverBase* Solver : AllSolvers)
		{
			// Send a copy of the material to each solver
			Solver->EnqueueCommandImmediate([InHandle, MaterialCopy = *Material, Solver]()
			{
				Solver->CastHelper([InHandle, &MaterialCopy](auto& Concrete)
				{
					Concrete.UpdateMaterial(InHandle,MaterialCopy);
				});
			});
		}
	}
}

void FChaosSolversModule::OnCreateMaterial(Chaos::FMaterialHandle InHandle)
{
	// Grab the material
	Chaos::FChaosPhysicsMaterial* Material = InHandle.Get();

	if(ensure(Material))
	{
		for(Chaos::FPhysicsSolverBase* Solver : AllSolvers)
		{
			// Send a copy of the material to each solver
			Solver->EnqueueCommandImmediate([InHandle, MaterialCopy = *Material, Solver]()
			{
				Solver->CastHelper([InHandle, &MaterialCopy](auto& Concrete)
				{
					Concrete.CreateMaterial(InHandle,MaterialCopy);
				});
			});
		}
	}
}

void FChaosSolversModule::OnDestroyMaterial(Chaos::FMaterialHandle InHandle)
{

	// Grab the material
	Chaos::FChaosPhysicsMaterial* Material = InHandle.Get();

	if(ensure(Material))
	{
		for(Chaos::FPhysicsSolverBase* Solver : AllSolvers)
		{
			// Notify each solver
			Solver->EnqueueCommandImmediate([InHandle, Solver]()
			{
				Solver->CastHelper([InHandle](auto& Concrete)
				{
					Concrete.DestroyMaterial(InHandle);
				});
			});
		}
	}
}

void FChaosSolversModule::OnUpdateMaterialMask(Chaos::FMaterialMaskHandle InHandle)
{

	// Grab the material
	Chaos::FChaosPhysicsMaterialMask* MaterialMask = InHandle.Get();

	if (ensure(MaterialMask))
	{
		for (Chaos::FPhysicsSolverBase* Solver : AllSolvers)
		{
			// Send a copy of the material to each solver
			Solver->EnqueueCommandImmediate([InHandle, MaterialMaskCopy = *MaterialMask, Solver]()
			{
				Solver->CastHelper([InHandle,&MaterialMaskCopy](auto& Concrete)
				{
					Concrete.UpdateMaterialMask(InHandle,MaterialMaskCopy);
				});
			});
		}
	}
}

void FChaosSolversModule::OnCreateMaterialMask(Chaos::FMaterialMaskHandle InHandle)
{

	// Grab the material
	Chaos::FChaosPhysicsMaterialMask* MaterialMask = InHandle.Get();

	if (ensure(MaterialMask))
	{
		for (Chaos::FPhysicsSolverBase* Solver : AllSolvers)
		{
			// Send a copy of the material to each solver
			Solver->EnqueueCommandImmediate([InHandle, MaterialMaskCopy = *MaterialMask, Solver]()
			{
				Solver->CastHelper([InHandle,&MaterialMaskCopy](auto& Concrete)
				{
					Concrete.CreateMaterialMask(InHandle,MaterialMaskCopy);
				});
			});
		}
	}
}

void FChaosSolversModule::OnDestroyMaterialMask(Chaos::FMaterialMaskHandle InHandle)
{

	// Grab the material
	Chaos::FChaosPhysicsMaterialMask* MaterialMask = InHandle.Get();

	if (ensure(MaterialMask))
	{
		for (Chaos::FPhysicsSolverBase* Solver : AllSolvers)
		{
			// Notify each solver
			Solver->EnqueueCommandImmediate([InHandle, Solver]()
			{
				Solver->CastHelper([InHandle](auto& Concrete)
				{
					Concrete.DestroyMaterialMask(InHandle);
				});
			});
		}
	}
}

const IChaosSettingsProvider& FChaosSolversModule::GetSettingsProvider() const
{
	return SettingsProvider ? *SettingsProvider : Chaos::GDefaultChaosSettings;
}