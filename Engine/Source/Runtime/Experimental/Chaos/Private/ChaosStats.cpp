// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosStats.h"

DEFINE_STAT(STAT_ChaosTick);
DEFINE_STAT(STAT_PhysicsAdvance);
DEFINE_STAT(STAT_SolverAdvance);
DEFINE_STAT(STAT_HandleSolverCommands);
DEFINE_STAT(STAT_IntegrateSolver);
DEFINE_STAT(STAT_SyncProxies);
DEFINE_STAT(STAT_PhysCommands);
DEFINE_STAT(STAT_TaskCommands);
DEFINE_STAT(STAT_WaitGlobalCommands);
DEFINE_STAT(STAT_KinematicUpdate);
DEFINE_STAT(STAT_BeginFrame);
DEFINE_STAT(STAT_EndFrame);
DEFINE_STAT(STAT_UpdateReverseMapping);
DEFINE_STAT(STAT_CollisionContactsCallback);
DEFINE_STAT(STAT_BreakingCallback);
DEFINE_STAT(STAT_TrailingCallback);
DEFINE_STAT(STAT_GCRaycast);
DEFINE_STAT(STAT_GCOverlap);
DEFINE_STAT(STAT_GCSweep);
DEFINE_STAT(STAT_GCCUpdateBounds);
DEFINE_STAT(STAT_GCUpdateFilterData);
DEFINE_STAT(STAT_GCCUGlobalMatrices);
DEFINE_STAT(STAT_GCPostPhysicsSync);
DEFINE_STAT(STAT_GCFullyDecayedBroadcast);
DEFINE_STAT(STAT_GCInitDynamicData);
DEFINE_STAT(STAT_GCTotalTransforms);
DEFINE_STAT(STAT_GCChangedTransforms);
DEFINE_STAT(STAT_GCReplicatedClusters);
DEFINE_STAT(STAT_GCReplicatedFractures);
DEFINE_STAT(STAT_LockWaits);
DEFINE_STAT(STAT_GeomBeginFrame);
DEFINE_STAT(STAT_SkelMeshUpdateAnim);
DEFINE_STAT(STAT_DispatchEventNotifies);
DEFINE_STAT(STAT_DispatchCollisionEvents);
DEFINE_STAT(STAT_DispatchBreakEvents);
DEFINE_STAT(STAT_DispatchCrumblingEvents);
DEFINE_STAT(STAT_BufferPhysicsResults);
DEFINE_STAT(STAT_FlipResults);
DEFINE_STAT(STAT_ProcessDeferredCreatePhysicsState);
DEFINE_STAT(STAT_SqUpdateMaterials);
DEFINE_STAT(STAT_CacheResultGeomCollection);
DEFINE_STAT(STAT_UpdateGeometryCollectionViews);
DEFINE_STAT(STAT_BufferPhysicsResultsParticleLoop);
DEFINE_STAT(STAT_CaptureSolverData);
DEFINE_STAT(STAT_CacheResultStaticMesh);
DEFINE_STAT(STAT_CaptureDisabledState);
DEFINE_STAT(STAT_CalcGlobalGCMatrices);
DEFINE_STAT(STAT_CalcGlobalGCBounds);
DEFINE_STAT(STAT_CalcParticleToWorld);
DEFINE_STAT(STAT_CreateBodies);
DEFINE_STAT(STAT_UpdateParams);
DEFINE_STAT(STAT_DisableCollisions);
DEFINE_STAT(STAT_EvolutionAndKinematicUpdate);
DEFINE_STAT(STAT_AdvanceEventWaits);
DEFINE_STAT(STAT_ResetCollisionRule);
DEFINE_STAT(STAT_EventDataGathering)
DEFINE_STAT(STAT_FillProducerData)
DEFINE_STAT(STAT_FlipBuffersIfRequired)
DEFINE_STAT(STAT_GatherCollisionEvent)
DEFINE_STAT(STAT_GatherBreakingEvent)
DEFINE_STAT(STAT_GatherTrailingEvent)
DEFINE_STAT(STAT_GatherSleepingEvent)
DEFINE_STAT(STAT_GatherCrumblingEvent)
DEFINE_STAT(STAT_AccelerationStructureReset);

DEFINE_STAT(STAT_FinalizeCallbacks);
DEFINE_STAT(STAT_ResetClusteringEvents);
DEFINE_STAT(STAT_RewindFinishFrame);
DEFINE_STAT(STAT_ResetMarshallingData);
DEFINE_STAT(STAT_ConditionalApplyRewind);
DEFINE_STAT(STAT_FinalizePullData);
DEFINE_STAT(STAT_DestroyPendingProxies);

DEFINE_STAT(STAT_ParamUpdateObject);
DEFINE_STAT(STAT_ParamUpdateField);
DEFINE_STAT(STAT_SyncEvents_GameThread);

DEFINE_STAT(STAT_PhysicsStatUpdate);
DEFINE_STAT(STAT_PhysicsThreadTime);
DEFINE_STAT(STAT_PhysicsThreadTimeEff);
DEFINE_STAT(STAT_PhysicsThreadFps);
DEFINE_STAT(STAT_PhysicsThreadFpsEff);

DEFINE_STAT(STAT_Scene_StartFrame);
DEFINE_STAT(STAT_Scene_EndFrame);

DEFINE_STAT(STAT_ParamUpdateField_Object);
DEFINE_STAT(STAT_ForceUpdateField_Object);
DEFINE_STAT(STAT_NiagaraUpdateField_Object);
DEFINE_STAT(STAT_ParamUpdateField_DynamicState);
DEFINE_STAT(STAT_ParamUpdateField_ActivateDisabled);
DEFINE_STAT(STAT_ParamUpdateField_ExternalClusterStrain);
DEFINE_STAT(STAT_ParamUpdateField_Kill);
DEFINE_STAT(STAT_ParamUpdateField_LinearVelocity);
DEFINE_STAT(STAT_ParamUpdateField_AngularVelocity);
DEFINE_STAT(STAT_ParamUpdateField_SleepingThreshold);
DEFINE_STAT(STAT_ParamUpdateField_DisableThreshold);
DEFINE_STAT(STAT_ParamUpdateField_InternalClusterStrain);
DEFINE_STAT(STAT_ParamUpdateField_CollisionGroup);
DEFINE_STAT(STAT_ParamUpdateField_PositionStatic);
DEFINE_STAT(STAT_ParamUpdateField_PositionTarget);
DEFINE_STAT(STAT_ParamUpdateField_PositionAnimated);
DEFINE_STAT(STAT_ParamUpdateField_DynamicConstraint);
DEFINE_STAT(STAT_ForceUpdateField_LinearForce);
DEFINE_STAT(STAT_ForceUpdateField_AngularTorque);
DEFINE_STAT(STAT_ForceUpdateField_LinearImpulse);

DEFINE_STAT(STAT_Collisions_Detect);
DEFINE_STAT(STAT_Collisions_ParticlePairBroadPhase);
DEFINE_STAT(STAT_Collisions_SpatialBroadPhase);
DEFINE_STAT(STAT_Collisions_BroadPhase);
DEFINE_STAT(STAT_Collisions_AssignMidPhases);
DEFINE_STAT(STAT_Collisions_NarrowPhase);
DEFINE_STAT(STAT_Collisions_GenerateCollisions);
DEFINE_STAT(STAT_Collisions_Gather);
DEFINE_STAT(STAT_Collisions_Scatter);
DEFINE_STAT(STAT_Collisions_Apply);
DEFINE_STAT(STAT_Collisions_ApplyPushOut);
DEFINE_STAT(STAT_Collisions_SimplifyConvexes);
