// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVisualDebugger/ChaosVDContextProvider.h"

#include "ChaosVDRuntimeModule.h"
#include "ChaosVisualDebugger/ChaosVisualDebuggerTrace.h"
#include "TraceBasedDebuggerRuntime.h"

#if WITH_CHAOS_VISUAL_DEBUGGER

UE_DEFINE_THREAD_SINGLETON_TLS(FChaosVDThreadContext, CHAOS_API)

FChaosVDThreadContext::~FChaosVDThreadContext()
{
	if (RecordingStoppedHandle.IsValid())
	{
		FChaosVDRuntimeModule::RemoveRecordingStopCallback(RecordingStoppedHandle);
	}
}

bool FChaosVDThreadContext::GetCurrentContext(FChaosVDContext& OutContext)
{
	if (LocalContextStack.Num() > 0)
	{
		OutContext = LocalContextStack.Top();
		return true;
	}

	return false;
}

const FChaosVDContext* FChaosVDThreadContext::GetCurrentContext(EChaosVDContextType Type)
{
	if (LocalContextStack.Num() > 0)
	{
		const FChaosVDContext* Context = &LocalContextStack.Top();

		return Context && Context->Type == static_cast<int32>(Type) ? Context : nullptr;
	}
	else
	{
		return nullptr;
	}
}

const FChaosVDContext* FChaosVDThreadContext::GetCurrentContext()
{
	return LocalContextStack.Num() > 0 ? &LocalContextStack.Top() : nullptr;
}

void FChaosVDThreadContext::PushContext(const FChaosVDContext& InContext)
{
	if (!FChaosVisualDebuggerTrace::IsTracing())
	{
		return;
	}

	LocalContextStack.Add(InContext);
}

void FChaosVDThreadContext::PopContext()
{
	if (LocalContextStack.Num() > 0)
	{
		LocalContextStack.Pop();
	}
}

TArray<uint8>& FChaosVDThreadContext::GetTLSDataBufferRef()
{
	checkf(BufferAccessCounter == 0, TEXT("The CVD buffer is already in use!"));

	if (!RecordingStoppedHandle.IsValid())
	{
		// If we access the buffer, make sure we subscribe to the recording stop callback
		// So we free any memory we keep allocated
		RecordingStoppedHandle = FChaosVDRuntimeModule::RegisterRecordingStopCallback(UE::TraceBasedDebuggers::FRecordingStateChangedDelegate::FDelegate::CreateLambda([this](){ CVDDataBuffer.Empty(); }));
	}
		
	return CVDDataBuffer;
}
#endif //WITH_CHAOS_VISUAL_DEBUGGER
