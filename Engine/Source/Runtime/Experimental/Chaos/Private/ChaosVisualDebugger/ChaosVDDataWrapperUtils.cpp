// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVisualDebugger/ChaosVDDataWrapperUtils.h"

#include "Chaos/SpatialAccelerationCollection.h"

#include "Chaos/PBDJointConstraints.h"
#include "DataWrappers/ChaosVDCharacterGroundConstraintDataWrappers.h"
#include "DataWrappers/ChaosVDJointDataWrappers.h"

#include "Chaos/Character/CharacterGroundConstraintContainer.h"
#include "Chaos/Collision/ParticlePairMidPhase.h"
#include "Chaos/ParticleHandle.h"
#include "DataWrappers/ChaosVDCollisionDataWrappers.h"
#include "DataWrappers/ChaosVDParticleDataWrapper.h"
#include "Math/UnitConversion.h"

namespace Chaos::VisualDebugger::Utils
{
	FGeometryParticle* GetPayloadForExternalThread(const Chaos::FAccelerationStructureHandle& Payload)
	{
		return Payload.GetExternalGeometryParticle_ExternalThread();	
	}

	/** Calculates and returns the current Game Thread bounds for the provided particle */
	FBox GetGeometricGTParticleBounds(const FGeometryParticle* GeometryParticle)
	{
		using namespace Chaos;

		FBox Bounds(ForceInitToZero);
		if (GeometryParticle)
		{
			const FShapesArray& Shapes = GeometryParticle->ShapesArray();
			for (const TUniquePtr<FPerShapeData>& Shape : Shapes)
			{
				Bounds += FBox(Shape->GetWorldSpaceShapeBounds().Min(), Shape->GetWorldSpaceShapeBounds().Max());
			}
		}
		return Bounds;
	}

	FBox GetPayloadBounds(const FAccelerationStructureHandle& Payload)
	{
		return GetGeometricGTParticleBounds(GetPayloadForExternalThread(Payload));
	}
}

void FChaosVDDataWrapperUtils::CopyManifoldPointsToDataWrapper(const Chaos::FManifoldPoint& InCopyFrom, FChaosVDManifoldPoint& OutCopyTo)
{
	OutCopyTo.bDisabled = InCopyFrom.Flags.bDisabled;
	OutCopyTo.bWasRestored = InCopyFrom.Flags.bWasRestored;
	OutCopyTo.bWasReplaced = InCopyFrom.Flags.bWasReplaced;
	OutCopyTo.bHasStaticFrictionAnchor = InCopyFrom.Flags.bHasStaticFrictionAnchor;
	OutCopyTo.TargetPhi = InCopyFrom.TargetPhi;
	OutCopyTo.InitialPhi = InCopyFrom.InitialPhi;

	Chaos::VisualDebugger::Utils::TransformStaticArray(InCopyFrom.ShapeAnchorPoints, OutCopyTo.ShapeAnchorPoints, &FChaosVDDataWrapperUtils::ConvertToFVector);
	Chaos::VisualDebugger::Utils::TransformStaticArray(InCopyFrom.InitialShapeContactPoints, OutCopyTo.InitialShapeContactPoints, &FChaosVDDataWrapperUtils::ConvertToFVector);
	Chaos::VisualDebugger::Utils::TransformStaticArray(InCopyFrom.ContactPoint.ShapeContactPoints, OutCopyTo.ContactPoint.ShapeContactPoints, &FChaosVDDataWrapperUtils::ConvertToFVector);

	OutCopyTo.ContactPoint.ShapeContactNormal = FVector(InCopyFrom.ContactPoint.ShapeContactNormal);
	OutCopyTo.ContactPoint.Phi = InCopyFrom.ContactPoint.Phi;
	OutCopyTo.ContactPoint.FaceIndex = InCopyFrom.ContactPoint.FaceIndex;
	OutCopyTo.ContactPoint.ContactType = static_cast<EChaosVDContactPointType>(InCopyFrom.ContactPoint.ContactType);
}

void FChaosVDDataWrapperUtils::CopyManifoldPointResultsToDataWrapper(const Chaos::FManifoldPointResult& InCopyFrom, FChaosVDManifoldPoint& OutCopyTo)
{
	OutCopyTo.NetPushOut =  FVector(InCopyFrom.NetPushOut);
	OutCopyTo.NetImpulse =  FVector(InCopyFrom.NetImpulse);
	OutCopyTo.bIsValid =  InCopyFrom.bIsValid;
	OutCopyTo.bInsideStaticFrictionCone =  InCopyFrom.bInsideStaticFrictionCone;
}

void FChaosVDDataWrapperUtils::CopyCollisionMaterialToDataWrapper(const Chaos::FPBDCollisionConstraintMaterial& InCopyFrom, FChaosVDCollisionMaterial& OutCopyTo)
{
	OutCopyTo.FaceIndex = InCopyFrom.FaceIndex;
	OutCopyTo.DynamicFriction = InCopyFrom.DynamicFriction;
	OutCopyTo.StaticFriction = InCopyFrom.StaticFriction;
	OutCopyTo.Restitution = InCopyFrom.Restitution;
	OutCopyTo.RestitutionThreshold = InCopyFrom.RestitutionThreshold;
	OutCopyTo.InvMassScale0 = InCopyFrom.InvMassScale0;
	OutCopyTo.InvMassScale1 = InCopyFrom.InvMassScale1;
	OutCopyTo.InvInertiaScale0 = InCopyFrom.InvInertiaScale0;
	OutCopyTo.InvInertiaScale1 = InCopyFrom.InvInertiaScale1;
}

FChaosVDParticleDataWrapper FChaosVDDataWrapperUtils::BuildParticleDataWrapperFromParticle(const Chaos::FGeometryParticleHandle* ParticleHandlePtr)
{
	check(ParticleHandlePtr);

	FChaosVDParticleDataWrapper WrappedParticleData;

	WrappedParticleData.ParticleContext = EChaosVDParticleContext::PhysicsThread;

	WrappedParticleData.ParticleIndex = ParticleHandlePtr->UniqueIdx().Idx;
	WrappedParticleData.Type =  static_cast<EChaosVDParticleType>(ParticleHandlePtr->Type);

	if (const Chaos::TKinematicGeometryParticleHandleImp<Chaos::FReal, 3, true>* KinematicParticle = ParticleHandlePtr->CastToKinematicParticle())
	{
		WrappedParticleData.ParticleVelocities.CopyFrom(*KinematicParticle);
		WrappedParticleData.ParticleKinematicTarget.CopyFrom(KinematicParticle->KinematicTarget());
	}

	if (const Chaos::TPBDRigidParticleHandleImp<Chaos::FReal, 3, true>* RigidParticle = ParticleHandlePtr->CastToRigidParticle())
	{
		WrappedParticleData.ParticlePositionRotation.CopyFrom<Chaos::TPBDRigidParticleHandleImp<Chaos::FReal, 3, true>, FChaosVDParticlePositionRotation::EAccessorType::PQ>(*RigidParticle);
		WrappedParticleData.ParticleVWSmooth.CopyFrom(*RigidParticle);
		WrappedParticleData.ParticleDynamics.CopyFrom(*RigidParticle);
		WrappedParticleData.ParticleDynamicsMisc.CopyFrom(*RigidParticle);
		WrappedParticleData.ParticleMassProps.CopyFrom(*RigidParticle);
	}
	else
	{
		WrappedParticleData.ParticlePositionRotation.CopyFrom<Chaos::FGeometryParticleHandle, FChaosVDParticlePositionRotation::EAccessorType::XR>(*ParticleHandlePtr);
	}

	WrappedParticleData.ParticleInflatedBounds.CopyFrom(*ParticleHandlePtr);

	if (const Chaos::TPBDRigidClusteredParticleHandleImp<Chaos::FReal, 3, true>* ClusteredParticle = ParticleHandlePtr->CastToClustered())
	{
		WrappedParticleData.ParticleCluster.CopyFrom(*ClusteredParticle);
	}

	WrappedParticleData.MarkAsValid();

	return MoveTemp(WrappedParticleData);
}

FChaosVDParticleDataWrapper FChaosVDDataWrapperUtils::BuildParticleDataWrapperFromGTParticle(const Chaos::FGeometryParticle* GTParticle)
{
	FChaosVDParticleDataWrapper WrappedParticleData;

	WrappedParticleData.ParticleContext = EChaosVDParticleContext::GameThread;

	WrappedParticleData.ParticleIndex = GTParticle->UniqueIdx().Idx;
	WrappedParticleData.Type =  static_cast<EChaosVDParticleType>(GTParticle->ObjectType());

	WrappedParticleData.ParticlePositionRotation.CopyFrom<Chaos::FGeometryParticle, FChaosVDParticlePositionRotation::EAccessorType::XR>(*GTParticle);

	if (const Chaos::TKinematicGeometryParticle<Chaos::FReal, 3>* KinematicParticle = GTParticle->CastToKinematicParticle())
	{
		WrappedParticleData.ParticleVelocities.CopyFrom(*KinematicParticle);
		WrappedParticleData.ParticleKinematicTarget.CopyFrom(KinematicParticle->KinematicTarget());
	}

	if (const Chaos::TPBDRigidParticle<Chaos::FReal, 3>* RigidParticle = GTParticle->CastToRigidParticle())
	{
		WrappedParticleData.ParticleDynamics.CopyFrom(*RigidParticle);
		WrappedParticleData.ParticleDynamicsMisc.CopyFrom(*RigidParticle);
		WrappedParticleData.ParticleMassProps.CopyFrom(*RigidParticle);
	}

	WrappedParticleData.DirtyFlagsBits = GTParticle->DirtyFlags().GetBits();

	// TODO: Add support for Cluster unions and Geometry Collections

	WrappedParticleData.MarkAsValid();

	return MoveTemp(WrappedParticleData);
}

FChaosVDConstraint FChaosVDDataWrapperUtils::BuildConstraintDataWrapperFromConstraint(const Chaos::FPBDCollisionConstraint& InConstraint)
{
	FChaosVDConstraint WrappedConstraintData;
	
	WrappedConstraintData.bIsCurrent = InConstraint.Flags.bIsCurrent;
	WrappedConstraintData.bDisabled = InConstraint.Flags.bDisabled;
	WrappedConstraintData.bUseManifold = InConstraint.Flags.bUseManifold;
	WrappedConstraintData.bUseIncrementalManifold = InConstraint.Flags.bUseIncrementalManifold;
	WrappedConstraintData.bCanRestoreManifold = InConstraint.Flags.bCanRestoreManifold;
	WrappedConstraintData.bWasManifoldRestored = InConstraint.Flags.bWasManifoldRestored;
	WrappedConstraintData.bIsQuadratic0 = InConstraint.Flags.bIsQuadratic0;
	WrappedConstraintData.bIsQuadratic1 = InConstraint.Flags.bIsQuadratic1;
	WrappedConstraintData.bIsProbe = InConstraint.Flags.bIsProbe;
	WrappedConstraintData.bCCDEnabled = InConstraint.Flags.bCCDEnabled;
	WrappedConstraintData.bCCDSweepEnabled = InConstraint.Flags.bCCDSweepEnabled;
	WrappedConstraintData.bModifierApplied = InConstraint.Flags.bModifierApplied;
	WrappedConstraintData.bMaterialSet = InConstraint.Flags.bMaterialSet;
	WrappedConstraintData.ShapesType = static_cast<EChaosVDContactShapesType>(InConstraint.ShapesType);
	WrappedConstraintData.CullDistance = InConstraint.CullDistance;
	WrappedConstraintData.CollisionTolerance = InConstraint.CollisionTolerance;
	WrappedConstraintData.ClosestManifoldPointIndex = InConstraint.ClosestManifoldPointIndex;
	WrappedConstraintData.ExpectedNumManifoldPoints = InConstraint.ExpectedNumManifoldPoints;
	WrappedConstraintData.Stiffness = InConstraint.Stiffness;
	WrappedConstraintData.MinInitialPhi = InConstraint.MinInitialPhi;
	WrappedConstraintData.InitialOverlapDepenetrationVelocity = InConstraint.InitialOverlapDepenetrationVelocity;
	WrappedConstraintData.CCDTimeOfImpact = InConstraint.CCDTimeOfImpact;
	WrappedConstraintData.CCDEnablePenetration = InConstraint.CCDEnablePenetration;
	WrappedConstraintData.CCDTargetPenetration = InConstraint.CCDTargetPenetration;
	
	CopyCollisionMaterialToDataWrapper(InConstraint.Material, WrappedConstraintData.Material);

	WrappedConstraintData.AccumulatedImpulse = FVector(InConstraint.AccumulatedImpulse);

	WrappedConstraintData.Particle0Index = InConstraint.GetParticle0()->UniqueIdx().Idx;
	WrappedConstraintData.Particle1Index = InConstraint.GetParticle1()->UniqueIdx().Idx;


	Chaos::VisualDebugger::Utils::TransformStaticArray(InConstraint.ShapeWorldTransforms, WrappedConstraintData.ShapeWorldTransforms, &Chaos::VisualDebugger::Utils::ConvertToFTransform);
	Chaos::VisualDebugger::Utils::TransformStaticArray(InConstraint.ImplicitTransform, WrappedConstraintData.ImplicitTransforms, &Chaos::VisualDebugger::Utils::ConvertToFTransform);

	WrappedConstraintData.CollisionMargins = TArray(InConstraint.CollisionMargins, std::size(InConstraint.CollisionMargins));
	WrappedConstraintData.LastShapeWorldPositionDelta = FVector(InConstraint.LastShapeWorldPositionDelta);
	WrappedConstraintData.LastShapeWorldRotationDelta = FQuat(InConstraint.LastShapeWorldRotationDelta);

	const int32 MaxManifoldPoints = InConstraint.ManifoldPoints.Num();
	WrappedConstraintData.ManifoldPoints.Reserve(MaxManifoldPoints);
	WrappedConstraintData.ManifoldPoints.SetNum(MaxManifoldPoints);

	for (int32 PointIndex = 0; PointIndex < MaxManifoldPoints; PointIndex++)
	{
		FChaosVDManifoldPoint& CurrentCVDMainFoldPoint = WrappedConstraintData.ManifoldPoints[PointIndex];

		if (PointIndex < InConstraint.SavedManifoldPoints.Num())
		{
			const Chaos::FSavedManifoldPoint& CurrentChaosSavedManifoldPoint = InConstraint.SavedManifoldPoints[PointIndex];

			Chaos::VisualDebugger::Utils::TransformStaticArray(CurrentChaosSavedManifoldPoint.ShapeContactPoints, CurrentCVDMainFoldPoint.ShapeContactPoints, &FChaosVDDataWrapperUtils::ConvertToFVector);
		}

		if (PointIndex < InConstraint.ManifoldPoints.Num())
		{
			const Chaos::FManifoldPoint& CurrentChaosMainFoldPoint = InConstraint.ManifoldPoints[PointIndex];
			CopyManifoldPointsToDataWrapper(CurrentChaosMainFoldPoint, CurrentCVDMainFoldPoint);
		}
		

		if (PointIndex < InConstraint.ManifoldPointResults.Num())
		{
			const Chaos::FManifoldPointResult& CurrentChaosMainFoldPointResult = InConstraint.ManifoldPointResults[PointIndex];
			CopyManifoldPointResultsToDataWrapper(CurrentChaosMainFoldPointResult, CurrentCVDMainFoldPoint);
		}
	}

	return MoveTemp(WrappedConstraintData);
}

FChaosVDParticlePairMidPhase FChaosVDDataWrapperUtils::BuildMidPhaseDataWrapperFromMidPhase(const Chaos::FParticlePairMidPhase& InMidPhase)
{
	FChaosVDParticlePairMidPhase WrappedMidPhaseData;
	
	WrappedMidPhaseData.bIsActive = InMidPhase.Flags.bIsActive;
	WrappedMidPhaseData.bIsCCD = InMidPhase.Flags.bIsCCD;
	WrappedMidPhaseData.bIsCCDActive = InMidPhase.Flags.bIsCCDActive;
	WrappedMidPhaseData.bIsSleeping = InMidPhase.Flags.bIsSleeping;
	WrappedMidPhaseData.bIsModified = InMidPhase.Flags.bIsModified;
	WrappedMidPhaseData.LastUsedEpoch = InMidPhase.LastUsedEpoch;

	WrappedMidPhaseData.Particle0Idx = InMidPhase.Particle0->UniqueIdx().Idx;
	WrappedMidPhaseData.Particle1Idx = InMidPhase.Particle1->UniqueIdx().Idx;

	InMidPhase.VisitConstCollisions([&WrappedMidPhaseData](const Chaos::FPBDCollisionConstraint& Constraint)
	{
		FChaosVDConstraint WrappedConstraintData = FChaosVDDataWrapperUtils::BuildConstraintDataWrapperFromConstraint(Constraint);
		WrappedMidPhaseData.Constraints.Add(MoveTemp(WrappedConstraintData));
		return Chaos::ECollisionVisitorResult::Continue;
	}, Chaos::ECollisionVisitorFlags::VisitAllCurrent);

	return MoveTemp(WrappedMidPhaseData);
}

#ifndef CVD_COPY_FIELD_TO_WRAPPER
	#define CVD_COPY_FIELD_TO_WRAPPER(Source,Target, Field) \
	Target.Field = Source.Field;
#endif

#ifndef CVD_COPY_FIELD_TO_WRAPPER_WITH_CAST
	#define CVD_COPY_FIELD_TO_WRAPPER_WITH_CAST(Source,Target, Field, TargetType) \
	Target.Field = static_cast<TargetType>(Source.Field);
#endif

#ifndef CVD_COPY_VECTOR_FIELD_TO_WRAPPER
	#define CVD_COPY_VECTOR_FIELD_TO_WRAPPER(Source,Target, Field) \
	Target.Field = FChaosVDDataWrapperUtils::ConvertToFVector(Source.Field);
#endif

void FChaosVDDataWrapperUtils::CopyJointConstraintSettingsToWrapper(const Chaos::FPBDJointSettings& InSettings, FChaosVDJointSettingsDataWrapper& InOutWrappedJointSettingsData)
{
	CVD_COPY_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, Stiffness);
	CVD_COPY_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, LinearProjection);
	CVD_COPY_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, AngularProjection);
	CVD_COPY_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, AngularProjection);
	CVD_COPY_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, TeleportDistance);
	CVD_COPY_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, TeleportAngle);
	CVD_COPY_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, ParentInvMassScale);
	CVD_COPY_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, bCollisionEnabled);
	CVD_COPY_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, bMassConditioningEnabled);
	CVD_COPY_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, bUseLinearSolver);
	CVD_COPY_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, bSoftLinearLimitsEnabled);
	CVD_COPY_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, bSoftTwistLimitsEnabled);
	CVD_COPY_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, bAngularSLerpPositionDriveEnabled);
	CVD_COPY_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, bAngularSLerpVelocityDriveEnabled);
	CVD_COPY_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, bAngularTwistPositionDriveEnabled);
	CVD_COPY_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, bAngularTwistVelocityDriveEnabled);
	CVD_COPY_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, bAngularSwingPositionDriveEnabled);
	CVD_COPY_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, bAngularSwingVelocityDriveEnabled);

	InOutWrappedJointSettingsData.LinearMotionTypes[0] = static_cast<EChaosVDJointMotionType>(InSettings.LinearMotionTypes[0]);
	InOutWrappedJointSettingsData.LinearMotionTypes[1] = static_cast<EChaosVDJointMotionType>(InSettings.LinearMotionTypes[1]);
	InOutWrappedJointSettingsData.LinearMotionTypes[2] = static_cast<EChaosVDJointMotionType>(InSettings.LinearMotionTypes[2]);

	CVD_COPY_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, LinearLimit);

	InOutWrappedJointSettingsData.AngularMotionTypes[0] = static_cast<EChaosVDJointMotionType>(InSettings.AngularMotionTypes[0]);
	InOutWrappedJointSettingsData.AngularMotionTypes[1] = static_cast<EChaosVDJointMotionType>(InSettings.AngularMotionTypes[1]);
	InOutWrappedJointSettingsData.AngularMotionTypes[2] = static_cast<EChaosVDJointMotionType>(InSettings.AngularMotionTypes[2]);

	CVD_COPY_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, AngularLimits);
	CVD_COPY_FIELD_TO_WRAPPER_WITH_CAST(InSettings, InOutWrappedJointSettingsData, LinearSoftForceMode, EChaosVDJointForceMode);
	CVD_COPY_FIELD_TO_WRAPPER_WITH_CAST(InSettings, InOutWrappedJointSettingsData, AngularSoftForceMode, EChaosVDJointForceMode);
	CVD_COPY_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, SoftLinearStiffness);
	CVD_COPY_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, SoftLinearDamping);
	CVD_COPY_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, SoftTwistStiffness);
	CVD_COPY_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, SoftTwistDamping);
	CVD_COPY_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, SoftSwingStiffness);
	CVD_COPY_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, SoftSwingDamping);
	CVD_COPY_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, LinearRestitution);
	CVD_COPY_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, TwistRestitution);
	CVD_COPY_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, TwistRestitution);
	CVD_COPY_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, SwingRestitution);
	CVD_COPY_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, LinearContactDistance);
	CVD_COPY_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, TwistContactDistance);
	CVD_COPY_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, SwingContactDistance);

	CVD_COPY_VECTOR_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, LinearDrivePositionTarget);
	CVD_COPY_VECTOR_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, LinearDriveVelocityTarget);
		
	InOutWrappedJointSettingsData.bLinearPositionDriveEnabled0 = InSettings.bLinearPositionDriveEnabled[0];
	InOutWrappedJointSettingsData.bLinearPositionDriveEnabled1 = InSettings.bLinearPositionDriveEnabled[1];
	InOutWrappedJointSettingsData.bLinearPositionDriveEnabled2 = InSettings.bLinearPositionDriveEnabled[2];

	InOutWrappedJointSettingsData.bLinearVelocityDriveEnabled0 = InSettings.bLinearVelocityDriveEnabled[0];
	InOutWrappedJointSettingsData.bLinearVelocityDriveEnabled1 = InSettings.bLinearVelocityDriveEnabled[1];
	InOutWrappedJointSettingsData.bLinearVelocityDriveEnabled2 = InSettings.bLinearVelocityDriveEnabled[2];

	CVD_COPY_FIELD_TO_WRAPPER_WITH_CAST(InSettings, InOutWrappedJointSettingsData, LinearDriveForceMode, EChaosVDJointForceMode);

	CVD_COPY_VECTOR_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, LinearDriveStiffness);
	CVD_COPY_VECTOR_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, LinearDriveDamping);
	CVD_COPY_VECTOR_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, LinearDriveMaxForce);
	CVD_COPY_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, AngularDrivePositionTarget);
	CVD_COPY_VECTOR_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, AngularDriveVelocityTarget);
	CVD_COPY_FIELD_TO_WRAPPER_WITH_CAST(InSettings, InOutWrappedJointSettingsData, AngularDriveForceMode, EChaosVDJointForceMode);
	CVD_COPY_VECTOR_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, AngularDriveStiffness);
	CVD_COPY_VECTOR_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, AngularDriveDamping);
	CVD_COPY_VECTOR_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, AngularDriveMaxTorque);

	CVD_COPY_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, LinearBreakForce);
	CVD_COPY_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, LinearPlasticityLimit);

	CVD_COPY_FIELD_TO_WRAPPER_WITH_CAST(InSettings, InOutWrappedJointSettingsData, LinearPlasticityType, EChaosVDPlasticityType);
		
	CVD_COPY_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, LinearPlasticityInitialDistanceSquared);
	CVD_COPY_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, AngularBreakTorque);
	CVD_COPY_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, AngularPlasticityLimit);
	CVD_COPY_FIELD_TO_WRAPPER(InSettings, InOutWrappedJointSettingsData, ContactTransferScale);
		
	InOutWrappedJointSettingsData.ConnectorTransforms[0] = InSettings.ConnectorTransforms[0];
	InOutWrappedJointSettingsData.ConnectorTransforms[1] = InSettings.ConnectorTransforms[1];
	
	InOutWrappedJointSettingsData.MarkAsValid();
}

FChaosVDJointConstraint FChaosVDDataWrapperUtils::BuildJointDataWrapper(const Chaos::FPBDJointConstraintHandle* ConstaintHanlde)
{
	FChaosVDJointConstraint WrappedJointData;

	if (ConstaintHanlde)
	{
		WrappedJointData.ConstraintIndex = ConstaintHanlde->GetConstraintIndex();

		if (Chaos::FGeometryParticleHandle* Particle0 = ConstaintHanlde->GetConstrainedParticles()[0])
		{
			WrappedJointData.ParticleParIndexes[0] = Particle0->UniqueIdx().Idx;
		}
		if (Chaos::FGeometryParticleHandle* Particle1 = ConstaintHanlde->GetConstrainedParticles()[1])
		{
			WrappedJointData.ParticleParIndexes[1] = Particle1->UniqueIdx().Idx;
		}

		WrappedJointData.PhysicsThreadJointState.bBroken = ConstaintHanlde->IsConstraintBroken();
		WrappedJointData.PhysicsThreadJointState.bBreaking = ConstaintHanlde->IsConstraintBreaking();
		WrappedJointData.PhysicsThreadJointState.bDisabled = !ConstaintHanlde->IsConstraintEnabled();
		WrappedJointData.PhysicsThreadJointState.bDriveTargetChanged = ConstaintHanlde->IsDriveTargetChanged();

		//TODO: Island related data getters are deprecated. We need to see where is bet to get that data now and if this should be recorded as part of the CVD Constraint wrapper
		//WrappedJointData.JointState.Color = ConstaintHanlde->GetConstraintColor();
		//WrappedJointData.JointState.Island = ConstaintHanlde->GetConstraintIsland();
		//WrappedJointData.JointState.IslandSize = ConstaintHanlde->GetConstraintIsland();
		
		WrappedJointData.PhysicsThreadJointState.bEnabledDuringResim = ConstaintHanlde->IsEnabledDuringResim();
		WrappedJointData.PhysicsThreadJointState.AngularImpulse = ConstaintHanlde->GetAngularImpulse();
		WrappedJointData.PhysicsThreadJointState.LinearImpulse = ConstaintHanlde->GetLinearImpulse();

		switch(ConstaintHanlde->ResimType())
		{
			case Chaos::EResimType::FullResim:
				WrappedJointData.PhysicsThreadJointState.ResimType = EChaosVDJointReSimType::FullResim;
				break;
			case Chaos::EResimType::ResimAsFollower:
				WrappedJointData.PhysicsThreadJointState.ResimType = EChaosVDJointReSimType::ResimAsFollower;
				break;
		}

		switch(ConstaintHanlde->SyncState())
		{
			case Chaos::ESyncState::InSync:
				WrappedJointData.PhysicsThreadJointState.SyncState = EChaosVDJointSyncType::InSync;
				break;
			case Chaos::ESyncState::HardDesync:
				WrappedJointData.PhysicsThreadJointState.SyncState = EChaosVDJointSyncType::HardDesync;
				break;
		}

		WrappedJointData.PhysicsThreadJointState.MarkAsValid();

		CopyJointConstraintSettingsToWrapper(ConstaintHanlde->GetJointSettings(), WrappedJointData.JointSettings);	 
	}
	
	WrappedJointData.MarkAsValid();
	
	return MoveTemp(WrappedJointData);
}

FChaosVDJointConstraint FChaosVDDataWrapperUtils::BuildGTJointDataWrapper(Chaos::FJointConstraint* GTConstraintPtr)
{
	FChaosVDJointConstraint WrappedJointData;

	if (!GTConstraintPtr)
	{
		return MoveTemp(WrappedJointData);
	}

	//TODO: Extract constrained particle indexes from the proxy pair. This will needed if we use this in CVD

	const Chaos::FJointConstraint::FOutputData& CurrentState = GTConstraintPtr->GetOutputData();

	WrappedJointData.GameThreadJointState.AngularViolation = CurrentState.AngularViolation;
	WrappedJointData.GameThreadJointState.bDriveTargetChanged = CurrentState.bDriveTargetChanged;
	WrappedJointData.GameThreadJointState.bIsBreaking = CurrentState.bIsBreaking;
	WrappedJointData.GameThreadJointState.bIsBroken = CurrentState.bIsBroken;
	WrappedJointData.GameThreadJointState.bIsViolating = CurrentState.bIsViolating;
	WrappedJointData.GameThreadJointState.LinearViolation = CurrentState.LinearViolation;
	WrappedJointData.GameThreadJointState.Force = CurrentState.Force;
	WrappedJointData.GameThreadJointState.Torque = CurrentState.Torque;

	CopyJointConstraintSettingsToWrapper(GTConstraintPtr->GetJointSettings(), WrappedJointData.JointSettings);

	return MoveTemp(WrappedJointData);
}


void FChaosVDDataWrapperUtils::CopyJointSettingsFromWrapper(const FChaosVDJointSettingsDataWrapper& InWrappedJointSettingsData, Chaos::FPBDJointSettings& NewSettings)
{
	CVD_COPY_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, Stiffness);
	CVD_COPY_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, LinearProjection);
	CVD_COPY_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, AngularProjection);
	CVD_COPY_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, AngularProjection);
	CVD_COPY_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, TeleportDistance);
	CVD_COPY_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, TeleportAngle);
	CVD_COPY_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, ParentInvMassScale);
	CVD_COPY_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, bCollisionEnabled);
	CVD_COPY_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, bMassConditioningEnabled);
	CVD_COPY_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, bUseLinearSolver);
	CVD_COPY_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, bSoftLinearLimitsEnabled);
	CVD_COPY_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, bSoftTwistLimitsEnabled);
	CVD_COPY_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, bAngularSLerpPositionDriveEnabled);
	CVD_COPY_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, bAngularSLerpVelocityDriveEnabled);
	CVD_COPY_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, bAngularTwistPositionDriveEnabled);
	CVD_COPY_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, bAngularTwistVelocityDriveEnabled);
	CVD_COPY_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, bAngularSwingPositionDriveEnabled);
	CVD_COPY_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, bAngularSwingVelocityDriveEnabled);

	NewSettings.LinearMotionTypes[0] = static_cast<Chaos::EJointMotionType>(InWrappedJointSettingsData.LinearMotionTypes[0]);
	NewSettings.LinearMotionTypes[1] = static_cast<Chaos::EJointMotionType>(InWrappedJointSettingsData.LinearMotionTypes[1]);
	NewSettings.LinearMotionTypes[2] = static_cast<Chaos::EJointMotionType>(InWrappedJointSettingsData.LinearMotionTypes[2]);

	CVD_COPY_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, LinearLimit);

	NewSettings.AngularMotionTypes[0] = static_cast<Chaos::EJointMotionType>(InWrappedJointSettingsData.AngularMotionTypes[0]);
	NewSettings.AngularMotionTypes[1] = static_cast<Chaos::EJointMotionType>(InWrappedJointSettingsData.AngularMotionTypes[1]);
	NewSettings.AngularMotionTypes[2] = static_cast<Chaos::EJointMotionType>(InWrappedJointSettingsData.AngularMotionTypes[2]);

	CVD_COPY_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, AngularLimits);
	CVD_COPY_FIELD_TO_WRAPPER_WITH_CAST(InWrappedJointSettingsData, NewSettings, LinearSoftForceMode, Chaos::EJointForceMode);
	CVD_COPY_FIELD_TO_WRAPPER_WITH_CAST(InWrappedJointSettingsData, NewSettings, AngularSoftForceMode, Chaos::EJointForceMode);
	CVD_COPY_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, SoftLinearStiffness);
	CVD_COPY_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, SoftLinearDamping);
	CVD_COPY_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, SoftTwistStiffness);
	CVD_COPY_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, SoftTwistDamping);
	CVD_COPY_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, SoftSwingStiffness);
	CVD_COPY_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, SoftSwingDamping);
	CVD_COPY_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, LinearRestitution);
	CVD_COPY_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, TwistRestitution);
	CVD_COPY_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, TwistRestitution);
	CVD_COPY_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, SwingRestitution);
	CVD_COPY_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, LinearContactDistance);
	CVD_COPY_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, TwistContactDistance);
	CVD_COPY_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, SwingContactDistance);

	CVD_COPY_VECTOR_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, LinearDrivePositionTarget);
	CVD_COPY_VECTOR_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, LinearDriveVelocityTarget);
	
	NewSettings.bLinearPositionDriveEnabled[0] = InWrappedJointSettingsData.bLinearPositionDriveEnabled0;
	NewSettings.bLinearPositionDriveEnabled[1] = InWrappedJointSettingsData.bLinearPositionDriveEnabled1;
	NewSettings.bLinearPositionDriveEnabled[2] = InWrappedJointSettingsData.bLinearPositionDriveEnabled2;

	NewSettings.bLinearVelocityDriveEnabled[0] = InWrappedJointSettingsData.bLinearVelocityDriveEnabled0;
	NewSettings.bLinearVelocityDriveEnabled[1] = InWrappedJointSettingsData.bLinearVelocityDriveEnabled1;
	NewSettings.bLinearVelocityDriveEnabled[2] = InWrappedJointSettingsData.bLinearVelocityDriveEnabled2;

	CVD_COPY_FIELD_TO_WRAPPER_WITH_CAST(InWrappedJointSettingsData, NewSettings, LinearDriveForceMode, Chaos::EJointForceMode);

	CVD_COPY_VECTOR_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, LinearDriveStiffness);
	CVD_COPY_VECTOR_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, LinearDriveDamping);
	CVD_COPY_VECTOR_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, LinearDriveMaxForce);
	CVD_COPY_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, AngularDrivePositionTarget);
	CVD_COPY_VECTOR_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, AngularDriveVelocityTarget);
	CVD_COPY_FIELD_TO_WRAPPER_WITH_CAST(InWrappedJointSettingsData, NewSettings, AngularDriveForceMode, Chaos::EJointForceMode);
	CVD_COPY_VECTOR_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, AngularDriveStiffness);
	CVD_COPY_VECTOR_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, AngularDriveDamping);
	CVD_COPY_VECTOR_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, AngularDriveMaxTorque);

	CVD_COPY_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, LinearBreakForce);
	CVD_COPY_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, LinearPlasticityLimit);

	CVD_COPY_FIELD_TO_WRAPPER_WITH_CAST(InWrappedJointSettingsData, NewSettings, LinearPlasticityType, Chaos::EPlasticityType);
	
	CVD_COPY_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, LinearPlasticityInitialDistanceSquared);
	CVD_COPY_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, AngularBreakTorque);
	CVD_COPY_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, AngularPlasticityLimit);
	CVD_COPY_FIELD_TO_WRAPPER(InWrappedJointSettingsData, NewSettings, ContactTransferScale);
	
	NewSettings.ConnectorTransforms[0] = InWrappedJointSettingsData.ConnectorTransforms[0];
	NewSettings.ConnectorTransforms[1] = InWrappedJointSettingsData.ConnectorTransforms[1];
}


void FChaosVDDataWrapperUtils::ApplyJointDataWrapperToHandle(Chaos::FPBDJointConstraintHandle* ConstaintHanlde, const FChaosVDJointConstraint& WrappedJointData)
{
	if (!ensure(ConstaintHanlde))
	{
		return;
	}

	using namespace Chaos;

	FPBDJointState NewState;
	FPBDJointSettings NewSettings;

	NewState.bBroken = WrappedJointData.PhysicsThreadJointState.bBroken;
	NewState.bBreaking = WrappedJointData.PhysicsThreadJointState.bBreaking;
	NewState.bDisabled = WrappedJointData.PhysicsThreadJointState.bDisabled;
	NewState.bDriveTargetChanged = WrappedJointData.PhysicsThreadJointState.bDriveTargetChanged;

	NewState.bEnabledDuringResim = WrappedJointData.PhysicsThreadJointState.bEnabledDuringResim;
	NewState.AngularImpulse = WrappedJointData.PhysicsThreadJointState.AngularImpulse;
	NewState.LinearImpulse = WrappedJointData.PhysicsThreadJointState.LinearImpulse;

	CVD_COPY_FIELD_TO_WRAPPER_WITH_CAST(WrappedJointData.PhysicsThreadJointState, NewState, ResimType, Chaos::EResimType);
	CVD_COPY_FIELD_TO_WRAPPER_WITH_CAST(WrappedJointData.PhysicsThreadJointState, NewState, SyncState, Chaos::ESyncState);

	CopyJointSettingsFromWrapper(WrappedJointData.JointSettings, NewSettings);

	ConstaintHanlde->SetSettings(MoveTemp(NewSettings));
	ConstaintHanlde->SetJointState(MoveTemp(NewState));
}

void FChaosVDDataWrapperUtils::ApplyJointDataWrapperGTConstraint(Chaos::FJointConstraint* ConstraintPtr, const FChaosVDJointConstraint& InWrappedJointData)
{
	if (!ensure(ConstraintPtr))
	{
		return;
	}

	using namespace Chaos;

	FJointConstraint::FOutputData& OutputStateDataRef = ConstraintPtr->GetOutputData();
	OutputStateDataRef.Torque = InWrappedJointData.GameThreadJointState.Torque;
	OutputStateDataRef.AngularViolation = InWrappedJointData.GameThreadJointState.AngularViolation;
	OutputStateDataRef.bDriveTargetChanged = InWrappedJointData.GameThreadJointState.bDriveTargetChanged;
	OutputStateDataRef.bIsBreaking = InWrappedJointData.GameThreadJointState.bIsBreaking;
	OutputStateDataRef.bIsBroken = InWrappedJointData.GameThreadJointState.bIsBroken;
	OutputStateDataRef.bIsViolating = InWrappedJointData.GameThreadJointState.bIsViolating;
	OutputStateDataRef.Force = InWrappedJointData.GameThreadJointState.Force;
	OutputStateDataRef.LinearViolation = InWrappedJointData.GameThreadJointState.LinearViolation;

	FPBDJointSettings NewSettings;
	CopyJointSettingsFromWrapper(InWrappedJointData.JointSettings, NewSettings);
	ConstraintPtr->SetJointSettings(MoveTemp(NewSettings));
}

FChaosVDCharacterGroundConstraint FChaosVDDataWrapperUtils::BuildCharacterGroundConstraintDataWrapper(const Chaos::FCharacterGroundConstraintHandle* ConstraintHandle)
{
	FChaosVDCharacterGroundConstraint WrappedConstraintData;

	if (ConstraintHandle)
	{
		if (ConstraintHandle->GetCharacterParticle())
		{
			WrappedConstraintData.CharacterParticleIndex = ConstraintHandle->GetCharacterParticle()->UniqueIdx().Idx;
		}
		else
		{
			WrappedConstraintData.CharacterParticleIndex = INDEX_NONE;
		}
		
		WrappedConstraintData.ConstraintIndex = WrappedConstraintData.CharacterParticleIndex; // TODO - add unique index to constraint handle

		if (ConstraintHandle->GetGroundParticle())
		{
			WrappedConstraintData.GroundParticleIndex = ConstraintHandle->GetGroundParticle()->UniqueIdx().Idx;
		}
		else
		{
			WrappedConstraintData.GroundParticleIndex = INDEX_NONE;
		}
		WrappedConstraintData.State.bDisabled = !ConstraintHandle->IsEnabled();

		//TODO: Island related data getters are deprecated. We need to see where is best to get that data now and if this should be recorded as part of the CVD Constraint wrapper
		//WrappedConstraintData.State.Color = ConstraintHandle->GetConstraintColor();
		//WrappedConstraintData.State.Island = ConstraintHandle->GetConstraintIsland();
		//WrappedConstraintData.State.IslandSize = ConstraintHandle->GetConstraintIsland();

		WrappedConstraintData.State.SolverAppliedForce = ConstraintHandle->GetSolverAppliedForce();
		WrappedConstraintData.State.SolverAppliedTorque = ConstraintHandle->GetSolverAppliedTorque();

		WrappedConstraintData.State.SolverAppliedForce.X = FUnitConversion::Convert(ConstraintHandle->GetSolverAppliedForce().X, EUnit::KilogramCentimetersPerSecondSquared, EUnit::Newtons);
		WrappedConstraintData.State.SolverAppliedForce.Y = FUnitConversion::Convert(ConstraintHandle->GetSolverAppliedForce().Y, EUnit::KilogramCentimetersPerSecondSquared, EUnit::Newtons);
		WrappedConstraintData.State.SolverAppliedForce.Z = FUnitConversion::Convert(ConstraintHandle->GetSolverAppliedForce().Z, EUnit::KilogramCentimetersPerSecondSquared, EUnit::Newtons);
		WrappedConstraintData.State.SolverAppliedTorque.X = FUnitConversion::Convert(ConstraintHandle->GetSolverAppliedTorque().X, EUnit::KilogramCentimetersSquaredPerSecondSquared, EUnit::NewtonMeters);
		WrappedConstraintData.State.SolverAppliedTorque.Y = FUnitConversion::Convert(ConstraintHandle->GetSolverAppliedTorque().Y, EUnit::KilogramCentimetersSquaredPerSecondSquared, EUnit::NewtonMeters);
		WrappedConstraintData.State.SolverAppliedTorque.Z = FUnitConversion::Convert(ConstraintHandle->GetSolverAppliedTorque().Z, EUnit::KilogramCentimetersSquaredPerSecondSquared, EUnit::NewtonMeters);

		WrappedConstraintData.State.MarkAsValid();

		CVD_COPY_FIELD_TO_WRAPPER(ConstraintHandle->GetSettings(), WrappedConstraintData.Settings, VerticalAxis);
		CVD_COPY_FIELD_TO_WRAPPER(ConstraintHandle->GetSettings(), WrappedConstraintData.Settings, TargetHeight);
		WrappedConstraintData.Settings.RadialForceLimit = FUnitConversion::Convert(ConstraintHandle->GetSettings().RadialForceLimit, EUnit::KilogramCentimetersPerSecondSquared, EUnit::Newtons);
		WrappedConstraintData.Settings.FrictionForceLimit = FUnitConversion::Convert(ConstraintHandle->GetSettings().FrictionForceLimit, EUnit::KilogramCentimetersPerSecondSquared, EUnit::Newtons);
		WrappedConstraintData.Settings.SwingTorqueLimit = FUnitConversion::Convert(ConstraintHandle->GetSettings().SwingTorqueLimit, EUnit::KilogramCentimetersSquaredPerSecondSquared, EUnit::NewtonMeters);
		WrappedConstraintData.Settings.TwistTorqueLimit = FUnitConversion::Convert(ConstraintHandle->GetSettings().TwistTorqueLimit, EUnit::KilogramCentimetersSquaredPerSecondSquared, EUnit::NewtonMeters);
		CVD_COPY_FIELD_TO_WRAPPER(ConstraintHandle->GetSettings(), WrappedConstraintData.Settings, CosMaxWalkableSlopeAngle);
		CVD_COPY_FIELD_TO_WRAPPER(ConstraintHandle->GetSettings(), WrappedConstraintData.Settings, DampingFactor);
		CVD_COPY_FIELD_TO_WRAPPER(ConstraintHandle->GetSettings(), WrappedConstraintData.Settings, AssumedOnGroundHeight);

		WrappedConstraintData.Settings.MarkAsValid();

		CVD_COPY_FIELD_TO_WRAPPER(ConstraintHandle->GetData(), WrappedConstraintData.Data, GroundNormal);
		CVD_COPY_FIELD_TO_WRAPPER(ConstraintHandle->GetData(), WrappedConstraintData.Data, TargetDeltaPosition);
		CVD_COPY_FIELD_TO_WRAPPER(ConstraintHandle->GetData(), WrappedConstraintData.Data, TargetDeltaFacing);
		CVD_COPY_FIELD_TO_WRAPPER(ConstraintHandle->GetData(), WrappedConstraintData.Data, GroundDistance);
		CVD_COPY_FIELD_TO_WRAPPER(ConstraintHandle->GetData(), WrappedConstraintData.Data, CosMaxWalkableSlopeAngle);

		WrappedConstraintData.Data.MarkAsValid();

	}

	WrappedConstraintData.MarkAsValid();

	return MoveTemp(WrappedConstraintData);
}

void FChaosVDDataWrapperUtils::CopyShapeDataToWrapper(const Chaos::FShapeInstancePtr& ShapeDataPtr, FChaosVDShapeCollisionData& OutCopyTo)
{
	const Chaos::FCollisionData& CollisionData = ShapeDataPtr->GetCollisionData();

	OutCopyTo.bQueryCollision = CollisionData.bQueryCollision;
	OutCopyTo.bIsProbe = CollisionData.bIsProbe;
	OutCopyTo.bSimCollision = CollisionData.bSimCollision;
	OutCopyTo.CollisionTraceType = static_cast<EChaosVDCollisionTraceFlag>(CollisionData.CollisionTraceType);

	const Chaos::Filter::FInstanceData InstanceData = CollisionData.GetFilterInstanceData();
	const Chaos::Filter::FShapeFilterData ShapeFilter = CollisionData.GetShapeFilterData();

	OutCopyTo.FilterData.OwnerId = InstanceData.GetOwnerId();
	OutCopyTo.FilterData.ComponentId = InstanceData.GetComponentId();
	OutCopyTo.FilterData.OverlapMask = ShapeFilter.GetOverlapChannels();
	OutCopyTo.FilterData.BlockMask = ShapeFilter.GetBlockChannels();
	OutCopyTo.FilterData.CollisionChannelMask = ShapeFilter.GetCollisionChannelMask();
	OutCopyTo.FilterData.SetMaskFilterAndFlags(ShapeFilter.GetMaskFilter(), (uint32)ShapeFilter.GetFlags());
}

void FChaosVDDataWrapperUtils::BuildDataWrapperFromAABBStructure(const Chaos::ISpatialAccelerationCollection<Chaos::FAccelerationStructureHandle, Chaos::FReal, 3>* SceneAccelerationStructures, int32 OwnerSolverID, TArray<FChaosVDAABBTreeDataWrapper>& OutAABBTrees)
{
	using namespace Chaos;

	if (!SceneAccelerationStructures)
	{
		return;
	}

	ISpatialAccelerationCollection<FAccelerationStructureHandle, FReal, 3>* MutableSceneAccelerationStructures = const_cast<ISpatialAccelerationCollection<Chaos::FAccelerationStructureHandle, Chaos::FReal, 3>*>(SceneAccelerationStructures);

	TArray<FSpatialAccelerationIdx> SpatialIndices = MutableSceneAccelerationStructures->GetAllSpatialIndices();
	OutAABBTrees.Reserve(SpatialIndices.Num());
	for (const FSpatialAccelerationIdx SpatialIndex : SpatialIndices)
	{
		const ISpatialAcceleration<FAccelerationStructureHandle, FReal, 3>* AccelerationStructure = MutableSceneAccelerationStructures->GetSubstructure(SpatialIndex);

		FChaosVDAABBTreeDataWrapper AABBTreDataWrapper;
		AABBTreDataWrapper.SolverId = OwnerSolverID;
		
		if (const TAABBTree<FAccelerationStructureHandle, TAABBTreeLeafArray<FAccelerationStructureHandle>>* AABBTree = AccelerationStructure->template As<TAABBTree<FAccelerationStructureHandle, TAABBTreeLeafArray<FAccelerationStructureHandle>>>())
		{
			BuildDataWrapperFromAABBStructure(*AABBTree, AABBTreDataWrapper);
			OutAABBTrees.Emplace(AABBTreDataWrapper);

			if (AABBTree->DirtyElementTree)
			{
				FChaosVDAABBTreeDataWrapper DirtyTreeWrapper;
				DirtyTreeWrapper.SolverId = OwnerSolverID;
				BuildDataWrapperFromAABBStructure(*AABBTree->DirtyElementTree, DirtyTreeWrapper);
				OutAABBTrees.Emplace(DirtyTreeWrapper);
			}
		}
		else if (const TAABBTree<FAccelerationStructureHandle, TBoundingVolume<FAccelerationStructureHandle>>* AABBTreeBV = AccelerationStructure->template As<TAABBTree<FAccelerationStructureHandle, TBoundingVolume<FAccelerationStructureHandle>>>())
		{
			BuildDataWrapperFromAABBStructure(*AABBTreeBV, AABBTreDataWrapper);
			OutAABBTrees.Emplace(AABBTreDataWrapper);

			if (AABBTreeBV->DirtyElementTree)
			{
				FChaosVDAABBTreeDataWrapper DirtyTreeWrapper;
				DirtyTreeWrapper.SolverId = OwnerSolverID;
				BuildDataWrapperFromAABBStructure(*AABBTreeBV->DirtyElementTree, DirtyTreeWrapper);
				OutAABBTrees.Emplace(DirtyTreeWrapper);
			}
		}
	}
}

void FChaosVDDataWrapperUtils::AddTreeLeaves(const TConstArrayView<Chaos::TAABBTreeLeafArray<Chaos::FAccelerationStructureHandle>>& LeavesContainer, FChaosVDAABBTreeDataWrapper& InOutAABBTreeWrapper)
{
	using namespace Chaos;

	InOutAABBTreeWrapper.LeavesNum = LeavesContainer.Num();
	InOutAABBTreeWrapper.TreeArrayLeafs.Reserve(InOutAABBTreeWrapper.LeavesNum);
	for (const TAABBTreeLeafArray<FAccelerationStructureHandle>& TreeArrayLeaf : LeavesContainer)
	{
		FChaosVDAABBTreeLeafDataWrapper CVDLeaf;
		CVDLeaf.Elements.Reserve(TreeArrayLeaf.Elems.Num());
			
		CVDLeaf.Bounds = ConvertToFBox(TreeArrayLeaf.GetBounds());

		for (const TPayloadBoundsElement<FAccelerationStructureHandle, FReal>& Elem : TreeArrayLeaf.Elems)
		{
			FChaosVDAABBTreePayloadBoundsElement CVELeafElement;
			CVELeafElement.ParticleIndex = GetUniqueIdx(Elem).Idx;
			CVELeafElement.Bounds = ConvertToFBox(Elem.Bounds);
			CVELeafElement.ActualBounds = VisualDebugger::Utils::GetPayloadBounds(Elem.Payload);

			CVELeafElement.MarkAsValid();
	
			CVDLeaf.Elements.Emplace(CVELeafElement);
		}
			
		CVDLeaf.MarkAsValid();

		InOutAABBTreeWrapper.TreeArrayLeafs.Emplace(MoveTemp(CVDLeaf));
	}
}

void FChaosVDDataWrapperUtils::AddTreeLeaves(const TConstArrayView<Chaos::TBoundingVolume<Chaos::FAccelerationStructureHandle>>& LeavesContainer, FChaosVDAABBTreeDataWrapper& InOutAABBTreeWrapper)
{
	using namespace Chaos;

	InOutAABBTreeWrapper.LeavesNum = LeavesContainer.Num();
	InOutAABBTreeWrapper.BoundingVolumeLeafs.Reserve(InOutAABBTreeWrapper.LeavesNum);

	for (const TBoundingVolume<FAccelerationStructureHandle>& BoundingVolumeLeaf : LeavesContainer)
	{
		TVector<int32, 3> ElementCounts = BoundingVolumeLeaf.MElements.Counts();

		FChaosVDBoundingVolumeDataWrapper CVDBoundingVolume;
		CVDBoundingVolume.MElementsCounts = FIntVector3(ElementCounts.X, ElementCounts.Y, ElementCounts.Z);

		CVDBoundingVolume.MaxPayloadBounds = BoundingVolumeLeaf.MaxPayloadBounds;

		typedef TBoundingVolume<FAccelerationStructureHandle>::FCellElement FCellElement;

		TConstArrayView<TArray<FCellElement>> ElementsFlatArray = MakeArrayView(BoundingVolumeLeaf.MElements.GetData(), ElementCounts.X * ElementCounts.Y * ElementCounts.Z);
			
		for (const TArray<FCellElement>& InCellElements : ElementsFlatArray)
		{
			TArray<FChaosVDBVCellElementDataWrapper> CellDataWrapper;
			Algo::Transform(InCellElements, CellDataWrapper, [](const FCellElement& InElement)
			{
				FChaosVDBVCellElementDataWrapper CVDCellElement;
				CVDCellElement.Bounds = ConvertToFBox(InElement.Bounds);
				CVDCellElement.ParticleIndex = GetUniqueIdx(InElement.Payload).Idx;
				CVDCellElement.StartIdx = FIntVector3(InElement.StartIdx.X, InElement.StartIdx.Y, InElement.StartIdx.Z);
				CVDCellElement.EndIdx = FIntVector3(InElement.EndIdx.X, InElement.EndIdx.Y, InElement.EndIdx.Z);
				CVDCellElement.MarkAsValid();
	
				return CVDCellElement;
			});
		}

		CVDBoundingVolume.MarkAsValid();
		
		InOutAABBTreeWrapper.BoundingVolumeLeafs.Emplace(MoveTemp(CVDBoundingVolume));
	}
}
