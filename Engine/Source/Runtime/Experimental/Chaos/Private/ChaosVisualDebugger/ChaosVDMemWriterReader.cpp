// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVisualDebugger/ChaosVDMemWriterReader.h"

namespace Chaos::VisualDebugger
{
	FStringView FChaosVDArchiveHeader::WrapperTypeName = TEXT("FChaosVDArchiveHeader");

	bool FChaosVDArchiveHeader::Serialize(FArchive& Ar)
	{
		Ar << EngineVersion;

		uint8 CustomVersionFormat = static_cast<uint8>(ECustomVersionSerializationFormat::Latest);
		Ar << CustomVersionFormat;
		CustomVersionContainer.Serialize(Ar, static_cast<ECustomVersionSerializationFormat>(CustomVersionFormat));

		return !Ar.IsError();
	}

	FChaosVDArchiveHeader FChaosVDArchiveHeader::Current()
	{
		FChaosVDArchiveHeader CurrentVersion;

		CurrentVersion.EngineVersion = FEngineVersion::Current();
		CurrentVersion.CustomVersionContainer = FCurrentCustomVersions::GetAll();

		return CurrentVersion;
	}
}
