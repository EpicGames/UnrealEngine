// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVisualDebugger/ChaosVDOptionalDataChannel.h"

#if WITH_CHAOS_VISUAL_DEBUGGER

#include "HAL/IConsoleManager.h"
#include "Internationalization/Text.h"
#include "Internationalization/Internationalization.h"
#include "Misc/LazySingleton.h"

FAutoConsoleCommand SetCVDDataChannelEnabledCommand(
	TEXT("p.Chaos.VD.SetCVDDataChannelEnabled"),
	TEXT("Turn on or off a CVD Data Channel. Argument 1 is true or false, Argument is a comma separated list of channel names. Example: p.Chaos.VD.SetCVDDataChannelEnabled true SceneQueries,Integrate"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		using namespace Chaos::VisualDebugger;

		constexpr int32 ExpectedArgumentsNum = 2;
		if (Args.Num() == ExpectedArgumentsNum)
		{
			const bool bNewEnabledState = Args[0] == TEXT("true");

			TArray<FString> ParsedChannels;
			ParseChannelListFromCommandArgument(ParsedChannels, Args[1]);

			for (const FString& ChannelName : ParsedChannels)
			{
				TWeakPtr<FChaosVDOptionalDataChannel> ChannelInstance = FChaosVDDataChannelsManager::Get().GetChannelById(FName(ChannelName));
				if (TSharedPtr<FChaosVDOptionalDataChannel> LockedChannelInstance = ChannelInstance.Pin())
				{
					LockedChannelInstance->SetChannelEnabled(bNewEnabledState);
				}
			}
		}
	})
);

namespace Chaos::VisualDebugger
{
	FChaosVDOptionalDataChannel::~FChaosVDOptionalDataChannel()
	{
		FChaosVDDataChannelsManager::Get().UnregisterChannel(*ChannelId);
	}
	
	void FChaosVDOptionalDataChannel::Initialize()
	{
		FChaosVDDataChannelsManager::Get().RegisterChannel(AsWeak());
	}

	const FText& FChaosVDOptionalDataChannel::GetDisplayName() const
	{
		return *LocalizableChannelName;
	}

	void FChaosVDOptionalDataChannel::SetChannelEnabled(bool bNewEnabled)
	{
		bIsEnabled = bNewEnabled;

		FChaosVDDataChannelsManager::Get().OnChannelStateChanged().Broadcast(AsWeak());
	}

	FChaosVDDataChannelsManager& FChaosVDDataChannelsManager::Get()
	{
		return TLazySingleton<FChaosVDDataChannelsManager>::Get();
	}

	void ParseChannelListFromCommandArgument(TArray<FString>& OutParsedChannelList, const FString& InCommandArgument)
	{
		constexpr int32 NumReserve = 10; // We don't really know how many channels will be requested, so start with something
		OutParsedChannelList.Reserve(NumReserve);
		InCommandArgument.ParseIntoArray(OutParsedChannelList,TEXT(","));
	}

	TSharedRef<FChaosVDOptionalDataChannel> CreateDataChannel(FName InChannelID, const TSharedRef<FText>& InDisplayName, EChaosVDDataChannelInitializationFlags InitializationFlags)
	{
		TSharedRef<FChaosVDOptionalDataChannel> NewChannel = MakeShared<FChaosVDOptionalDataChannel>(MakeShared<FName>(InChannelID), InDisplayName, InitializationFlags);
		NewChannel->Initialize();

		return NewChannel;
	}

	TSharedPtr<FChaosVDOptionalDataChannel> CreateDataChannelShared(FName InChannelID, const TSharedRef<FText>& InDisplayName, EChaosVDDataChannelInitializationFlags InitializationFlags)
	{
		TSharedPtr<FChaosVDOptionalDataChannel> NewChannel = MakeShared<FChaosVDOptionalDataChannel>(MakeShared<FName>(InChannelID), InDisplayName, InitializationFlags);
		NewChannel->Initialize();

		return NewChannel;
	}
}

CVD_DEFINE_OPTIONAL_DATA_CHANNEL(Default, EChaosVDDataChannelInitializationFlags::StartEnabled);
CVD_DEFINE_OPTIONAL_DATA_CHANNEL(EvolutionStart, EChaosVDDataChannelInitializationFlags::StartEnabled | EChaosVDDataChannelInitializationFlags::CanChangeEnabledState);
CVD_DEFINE_OPTIONAL_DATA_CHANNEL(PostIntegrate, EChaosVDDataChannelInitializationFlags::CanChangeEnabledState);
CVD_DEFINE_OPTIONAL_DATA_CHANNEL(PreConstraintSolve, EChaosVDDataChannelInitializationFlags::CanChangeEnabledState);
CVD_DEFINE_OPTIONAL_DATA_CHANNEL(CollisionDetectionBroadPhase, EChaosVDDataChannelInitializationFlags::CanChangeEnabledState);
CVD_DEFINE_OPTIONAL_DATA_CHANNEL(CollisionDetectionNarrowPhase, EChaosVDDataChannelInitializationFlags::CanChangeEnabledState);
CVD_DEFINE_OPTIONAL_DATA_CHANNEL(PostConstraintSolve, EChaosVDDataChannelInitializationFlags::CanChangeEnabledState);
CVD_DEFINE_OPTIONAL_DATA_CHANNEL(EndOfEvolutionCollisionConstraints, EChaosVDDataChannelInitializationFlags::StartEnabled | EChaosVDDataChannelInitializationFlags::CanChangeEnabledState);
CVD_DEFINE_OPTIONAL_DATA_CHANNEL(EvolutionEnd, EChaosVDDataChannelInitializationFlags::StartEnabled); // Intentionally not allowing changing the enabled state because otherwise there might be nothing to visualize
CVD_DEFINE_OPTIONAL_DATA_CHANNEL(SceneQueries, EChaosVDDataChannelInitializationFlags::StartEnabled | EChaosVDDataChannelInitializationFlags::CanChangeEnabledState);
CVD_DEFINE_OPTIONAL_DATA_CHANNEL(JointLinearConstraints, EChaosVDDataChannelInitializationFlags::CanChangeEnabledState);
CVD_DEFINE_OPTIONAL_DATA_CHANNEL(JointNonLinearConstraints, EChaosVDDataChannelInitializationFlags::CanChangeEnabledState);
CVD_DEFINE_OPTIONAL_DATA_CHANNEL(CharacterGroundConstraints, EChaosVDDataChannelInitializationFlags::CanChangeEnabledState);
CVD_DEFINE_OPTIONAL_DATA_CHANNEL(AccelerationStructures, EChaosVDDataChannelInitializationFlags::CanChangeEnabledState);
CVD_DEFINE_OPTIONAL_DATA_CHANNEL(ClusterParticlesChildData, EChaosVDDataChannelInitializationFlags::CanChangeEnabledState);
CVD_DEFINE_OPTIONAL_DATA_CHANNEL(CameraPaths, EChaosVDDataChannelInitializationFlags::StartEnabled | EChaosVDDataChannelInitializationFlags::CanChangeEnabledState);
CVD_DEFINE_OPTIONAL_DATA_CHANNEL(ResimDetails, EChaosVDDataChannelInitializationFlags::CanChangeEnabledState);

#endif
