// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVisualDebugger/ChaosVDParticleExtraData.h"

#include "ChaosVisualDebugger/ChaosVDMemWriterReader.h"
#include "ChaosVisualDebugger/ChaosVDStructCollectionMemWriterReader.h"
#include "ChaosVisualDebugger/ChaosVisualDebuggerTrace.h"
#include "Logging/LogMacros.h"
#include "UObject/Class.h"

DEFINE_LOG_CATEGORY_STATIC(LogChaosVDParticleExtraData, Log, All);

namespace Chaos::VisualDebugger
{
	FStringView FChaosVDParticleExtraData::WrapperTypeName = TEXT("FChaosVDParticleExtraData");

	bool FChaosVDExtraDataStructEntry::Serialize(FArchive& Ar)
	{
		Ar << StructTypePath;
		uint8 ModeVal = static_cast<uint8>(SerializationMode);
		Ar << ModeVal;
		if (Ar.IsLoading())
		{
			SerializationMode = static_cast<EChaosVDExtraDataSerializationMode>(ModeVal);
		}
		Ar << Bytes;
		return !Ar.IsError();
	}

	void FChaosVDExtraDataStructEntry::SerializeFrom(UScriptStruct* Struct, const void* Data,
		const TSharedRef<FChaosVDSerializableNameTable>& NameTable,
		EChaosVDExtraDataSerializationMode Mode)
	{
		SerializationMode = Mode;
		FChaosVDStructCollectionMemoryWriter Writer(Bytes, NameTable);
		if (Mode == EChaosVDExtraDataSerializationMode::SerializeBin)
		{
			Struct->SerializeBin(Writer, const_cast<void*>(Data));
		}
		else
		{
			Struct->SerializeItem(Writer, const_cast<void*>(Data), nullptr);
		}
	}

	void FChaosVDExtraDataCategory::AddEntry(UScriptStruct* Struct, const void* Data, EChaosVDExtraDataSerializationMode Mode)
	{
#if WITH_CHAOS_VISUAL_DEBUGGER
		FChaosVDExtraDataStructEntry& Entry = Entries.AddDefaulted_GetRef();
		Entry.StructTypePath = FName(Struct->GetPathName());
		Entry.SerializeFrom(Struct, Data, FChaosVisualDebuggerTrace::GetNameTableInstance(), Mode);
#endif
	}

	bool FChaosVDExtraDataCategory::Serialize(FArchive& Ar)
	{
		Ar << CategoryName;
		Ar << SourceChannelId;

		int32 NumEntries = Entries.Num();
		Ar << NumEntries;
		if (Ar.IsLoading())
		{
			constexpr int32 MaxExpectedEntries = 256;
			if (NumEntries < 0 || NumEntries > MaxExpectedEntries)
			{
				UE_LOGF(LogChaosVDParticleExtraData, Warning,
					"FChaosVDExtraDataCategory::Serialize - Invalid NumEntries (%d) for category '%ls', expected 0..%d. Data may be corrupt.",
					NumEntries, *CategoryName.ToString(), MaxExpectedEntries);
				Ar.SetError();
				return false;
			}
			Entries.SetNum(NumEntries);
		}
		for (FChaosVDExtraDataStructEntry& Entry : Entries)
		{
			Entry.Serialize(Ar);
		}
		return !Ar.IsError();
	}

	bool FChaosVDParticleExtraData::Serialize(FArchive& Ar)
	{
		Ar << SolverID;
		Ar << ParticleID;

		int32 NumCategories = Categories.Num();
		Ar << NumCategories;
		if (Ar.IsLoading())
		{
			constexpr int32 MaxExpectedCategories = 256;
			if (NumCategories < 0 || NumCategories > MaxExpectedCategories)
			{
				UE_LOGF(LogChaosVDParticleExtraData, Warning,
					"FChaosVDParticleExtraData::Serialize - Invalid NumCategories (%d), expected 0..%d. Data may be corrupt.",
					NumCategories, MaxExpectedCategories);
				Ar.SetError();
				return false;
			}
			Categories.SetNum(NumCategories);
		}
		for (FChaosVDExtraDataCategory& Category : Categories)
		{
			Category.Serialize(Ar);
		}
		return !Ar.IsError();
	}

	void TraceChaosVDParticleExtraData(int32 SolverID, int32 ParticleID, FChaosVDParticleExtraData& InExtraData)
	{
#if WITH_CHAOS_VISUAL_DEBUGGER
		if (!FChaosVisualDebuggerTrace::IsTracing())
		{
			return;
		}

		InExtraData.SolverID = SolverID;
		InExtraData.ParticleID = ParticleID;

		TArray<uint8> Buffer;
		const TSharedRef<FChaosVDSerializableNameTable> NameTable = FChaosVisualDebuggerTrace::GetNameTableInstance();
		FChaosVDMemoryWriter Writer(Buffer, NameTable);

		InExtraData.Serialize(Writer);

		FChaosVisualDebuggerTrace::TraceBinaryData(Buffer, FChaosVDParticleExtraData::WrapperTypeName);
#endif
	}
}
