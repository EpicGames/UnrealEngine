// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVisualDebugger/ChaosVDSerializedNameTable.h"

#include "ChaosVisualDebugger/ChaosVDTraceMacros.h"
#include "ChaosVisualDebugger/ChaosVisualDebuggerTrace.h"
#include "Misc/ScopeRWLock.h"
#include "Serialization/MemoryWriter.h"

namespace Chaos::VisualDebugger
{
	FStringView FChaosVDSerializedNameEntry::WrapperTypeName = TEXT("FChaosVDSerializedNameEntry");

	uint64 FChaosVDSerializableNameTable::AddNameToTable(FName Name)
	{
		const uint64 NameID = Name.ToUnstableInt();
		if (NameID == 0)
		{
			// 0 means an empty name, so don't bother reading the name table
			return NameID;
		}

		{
			UE::TReadScopeLock ReadLock(NamesByIDLock);
			if (NamesByID.Contains(NameID))
			{
				return NameID;
			}
		}
		{
			UE::TWriteScopeLock WriteLock(NamesByIDLock);			
			NamesByID.Add(NameID, Name);
		}

		{
			FChaosVDSerializedNameEntry NameEntry {NameID, Name.GetNumber(), Name.GetPlainNameString()};

			FMemMark StackMarker(FMemStack::Get());
			TArray<uint8, TInlineAllocator<256, TMemStackAllocator<>>> NameBuffer;

			TMemoryWriterBase MemWriterAr(NameBuffer);
			MemWriterAr.SetShouldSkipUpdateCustomVersion(true);

			MemWriterAr << NameEntry;

			CVD_TRACE_BINARY_DATA(NameBuffer, FChaosVDSerializedNameEntry::WrapperTypeName);
		}
		
		return NameID;
	}

	uint64 FChaosVDSerializableNameTable::AddNameToTable(const FChaosVDSerializedNameEntry& InNameEntry)
	{
		UE::TWriteScopeLock WriteLock(NamesByIDLock);

		int32 NameNumber = InNameEntry.NameNumber == NAME_NO_NUMBER ? NAME_NO_NUMBER_INTERNAL : InNameEntry.NameNumber;

		NamesByID.Add(InNameEntry.NameID, FName(InNameEntry.Name, NameNumber));

		return InNameEntry.NameID;
	}

	FName FChaosVDSerializableNameTable::GetNameFromTable(uint64 NameID)
	{
		if (NameID == 0)
		{
			// 0 means an empty name, so don't bother reading the name table
			return FName();
		}

		{
			UE::TReadScopeLock ReadLock(NamesByIDLock);
			if (const FName* FoundName = NamesByID.Find(NameID))
			{
				return *FoundName;
			}
		}

		return FName();
	}

	void FChaosVDSerializableNameTable::ResetTable()
	{
		{
			UE::TWriteScopeLock WriteLock(NamesByIDLock);
			NamesByID.Empty();
		}
	}
}
