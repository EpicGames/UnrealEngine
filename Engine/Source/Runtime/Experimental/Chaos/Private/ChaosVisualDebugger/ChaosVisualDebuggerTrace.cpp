// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVisualDebugger/ChaosVisualDebuggerTrace.h"

#if WITH_CHAOS_VISUAL_DEBUGGER

#include "Chaos/Character/CharacterGroundConstraintContainer.h"
#include "Chaos/Framework/PhysicsSolverBase.h"
#include "Chaos/ImplicitObject.h"
#include "Chaos/ISpatialAccelerationCollection.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/PBDCollisionConstraints.h"
#include "Chaos/PBDJointConstraints.h"
#include "Chaos/PBDRigidClustering.h"
#include "Chaos/PBDRigidsSOAs.h"
#include "ChaosVDRuntimeModule.h"
#include "ChaosVisualDebugger/ChaosVDDataWrapperUtils.h"
#include "ChaosVisualDebugger/ChaosVDMemWriterReader.h"
#include "ChaosVisualDebugger/ChaosVDSerializedNameTable.h"
#include "Compression/OodleDataCompressionUtil.h"
#include "Containers/StripedMap.h"
#include "DataWrappers/ChaosVDCharacterGroundConstraintDataWrappers.h"
#include "DataWrappers/ChaosVDCollisionDataWrappers.h"
#include "DataWrappers/ChaosVDDebugShapeDataWrapper.h"
#include "DataWrappers/ChaosVDImplicitObjectDataWrapper.h"
#include "DataWrappers/ChaosVDJointDataWrappers.h"
#include "DataWrappers/ChaosVDParticleDataWrapper.h"
#include "DataWrappers/ChaosVDQueryDataWrappers.h"
#include "DataWrappers/ChaosVDCameraDataWrapper.h"
#include "Engine/Engine.h"
#include "HAL/CriticalSection.h"
#include "Misc/ScopeRWLock.h"
#include "Templates/UniquePtr.h"
#include "TraceBasedDebuggerRuntime.h"
#include "UObject/ObjectMacros.h"

static_assert(TRACE_BASED_DEBUGGERS_WITHOUT_TRACE || UE_TRACE_MINIMAL_ENABLED || UE_TRACE_ENABLED,
	"Chaos Visual Debugger support requires Trace or Trace minimal support."
	" Please enable trace support with the Unreal Build Tool flag bEnableTrace, disable cvd support with bCompileChaosVisualDebuggerSupport"
	" or allows CVD support without trace by defining TRACE_BASED_DEBUGGERS_WITHOUT_TRACE to 1");

UE_TRACE_MINIMAL_EVENT_DEFINE(ChaosVDLogger, ChaosVDSolverFrameStart)

UE_TRACE_MINIMAL_EVENT_DEFINE(ChaosVDLogger, ChaosVDSolverFrameEnd)

UE_TRACE_MINIMAL_CHANNEL_DEFINE(ChaosVDChannel, "Chaos physics solver frames, simulation steps, particles, constraints (collision, joints, character ground), scene queries, implicit geometry, and debug shapes. Required for the Chaos Visual Debugger (CVD) tool.");
UE_TRACE_MINIMAL_EVENT_DEFINE(ChaosVDLogger, ChaosVDParticle)
UE_TRACE_MINIMAL_EVENT_DEFINE(ChaosVDLogger, ChaosVDParticleDestroyed)
UE_TRACE_MINIMAL_EVENT_DEFINE(ChaosVDLogger, ChaosVDSolverStepStart)
UE_TRACE_MINIMAL_EVENT_DEFINE(ChaosVDLogger, ChaosVDSolverStepEnd)
UE_TRACE_MINIMAL_EVENT_DEFINE(ChaosVDLogger, ChaosVDBinaryDataStart)
UE_TRACE_MINIMAL_EVENT_DEFINE(ChaosVDLogger, ChaosVDBinaryDataContent)
UE_TRACE_MINIMAL_EVENT_DEFINE(ChaosVDLogger, ChaosVDBinaryDataEnd)
UE_TRACE_MINIMAL_EVENT_DEFINE(ChaosVDLogger, ChaosVDSolverSimulationSpace)
UE_TRACE_MINIMAL_EVENT_DEFINE(ChaosVDLogger, ChaosVDDummyEvent)
UE_TRACE_MINIMAL_EVENT_DEFINE(ChaosVDLogger, ChaosVDNonSolverLocation)
UE_TRACE_MINIMAL_EVENT_DEFINE(ChaosVDLogger, ChaosVDNonSolverTransform)
UE_TRACE_MINIMAL_EVENT_DEFINE(ChaosVDLogger, ChaosVDNetworkTickOffset)
UE_TRACE_MINIMAL_EVENT_DEFINE(ChaosVDLogger, ChaosVDRolledBackDataID)
UE_TRACE_MINIMAL_EVENT_DEFINE(ChaosVDLogger, ChaosVDUsesAutoRTFM)
UE_TRACE_MINIMAL_EVENT_DEFINE(ChaosVDLogger, ChaosVDMiscHeaderData)
UE_TRACE_MINIMAL_EVENT_DEFINE(ChaosVDLogger, ChaosVDTraceRelevancyVolume)

namespace Chaos::VisualDebugger::Cvars
{
	static bool bCompressBinaryData = false;
	FAutoConsoleVariableRef CVarChaosVDCompressBinaryData(
	TEXT("p.Chaos.VD.CompressBinaryData"),
	bCompressBinaryData,
	TEXT("If true, serialized binary data will be compressed using Oodle on the fly before being traced"));

	static int32 CompressionMode = 2;
	FAutoConsoleVariableRef CVarChaosVDCompressionMode(
	TEXT("p.Chaos.VD.CompressionMode"),
	CompressionMode,
	TEXT("Oodle compression mode to use, 4 is by default which equals to ECompressionLevel::VeryFast"));

	static bool bUsesSharedGeometryArchiveContext = true;
	FAutoConsoleVariableRef CVarChaosVDUsesSharedGeometryArchiveContext(
	TEXT("p.Chaos.VD.UseSharedGeometryArchiveContext"),
	bUsesSharedGeometryArchiveContext,
	TEXT("If set to true, CVD will serialize collision geometry using a shared archive context to de-duplicate geometry during serialization instead of during deserialization"));
}

namespace Chaos::VD
{
	FRecordingSessionState::FRecordingSessionState() : CVDNameTable(MakeShared<FChaosVDSerializableNameTable>()),
	TraceRelevancyVolume(FBox(EForceInit::ForceInitToZero)),
	bIsTracing(false), ArchiveContext(MakeShared<FChaosArchiveContext>())
	{
	}

	namespace Private
	{
		FChaosVDParticleMetadata GenerateParticleMetadata(const FGeometryParticleHandle* ParticleHandle, FRecordingSessionState& SessionState)
		{
			if (!ParticleHandle)
			{
				return FChaosVDParticleMetadata();
			}

			if (SessionState.ExternalParticleMetadataGenerator.IsBound())
			{
				FChaosVDParticleMetadata GeneratedMetadata = SessionState.ExternalParticleMetadataGenerator.Execute(ParticleHandle->PhysicsProxy(), ParticleHandle);
				GeneratedMetadata.MetadataID = SessionState.GenerateUniqueID();
				return GeneratedMetadata;
			}
			else
			{
				return FChaosVDParticleMetadata();
			}
		}
	}
}

Chaos::VD::FRecordingSessionState FChaosVisualDebuggerTrace::SessionState = Chaos::VD::FRecordingSessionState();

void FChaosVisualDebuggerTrace::TraceParticle(const Chaos::FGeometryParticleHandle* ParticleHandle)
{
	using namespace Chaos::VisualDebugger::Utils;
	if (!IsTracing())
	{
		return;
	}

	const FChaosVDContext* CVDContextData = FChaosVDThreadContext::Get().GetCurrentContext(EChaosVDContextType::Solver);
	if (!IsContextEnabledAndValid(CVDContextData))
	{
		return;
	}

	TraceParticle(const_cast<Chaos::FGeometryParticleHandle*>(ParticleHandle), *CVDContextData);
}

void FChaosVisualDebuggerTrace::TraceParticle(Chaos::FGeometryParticleHandle* ParticleHandle, const FChaosVDContext& ContextData)
{
	if (!IsTracing())
	{
		return;
	}

	if (!ParticleHandle)
	{
		UE_LOGF(LogChaos, Warning, "Tried to Trace a null particle %s", __FUNCTION__);
		return;
	}

	if (!IsInRelevancyVolume(ParticleHandle))
	{
		return;
	}

	const Chaos::FImplicitObjectRef ParticleGeometry = ParticleHandle->GetGeometry();

	FChaosVDParticleDataWrapper ParticleDataWrapper = FChaosVDDataWrapperUtils::BuildParticleDataWrapperFromParticle(ParticleHandle);
	ParticleDataWrapper.GeometryHash = TraceImplicitObject(ParticleGeometry);
	ParticleDataWrapper.MetadataId = TraceParticleMetadata(ParticleHandle);
	ParticleDataWrapper.SolverID = ContextData.Id;
	
	const Chaos::FShapeInstanceArray& ShapesInstancesArray = ParticleHandle->ShapeInstances();
	ParticleDataWrapper.CollisionDataPerShape.Reserve(ShapesInstancesArray.Num());
	
	for (const Chaos::FShapeInstancePtr& ShapeData : ShapesInstancesArray)
	{
		FChaosVDShapeCollisionData CVDCollisionData;
		FChaosVDDataWrapperUtils::CopyShapeDataToWrapper(ShapeData, CVDCollisionData);
		ParticleDataWrapper.CollisionDataPerShape.Add(MoveTemp(CVDCollisionData));
	}
	
	FChaosVDScopedTLSBufferAccessor TLSDataBuffer;

	Chaos::VisualDebugger::WriteDataToBuffer(TLSDataBuffer.BufferRef, ParticleDataWrapper);

	TraceBinaryData(TLSDataBuffer.BufferRef, FChaosVDParticleDataWrapper::WrapperTypeName);
}

void FChaosVisualDebuggerTrace::TraceParticles(const Chaos::TGeometryParticleHandles<Chaos::FReal, 3>& ParticleHandles)
{
	using namespace Chaos::VisualDebugger::Utils;
	if (!IsTracing())
	{
		return;
	}

	const FChaosVDContext* CVDContextData = FChaosVDThreadContext::Get().GetCurrentContext(EChaosVDContextType::Solver);
	if (!IsContextEnabledAndValid(CVDContextData))
	{
		return;
	}

	ParallelFor(ParticleHandles.Size(),[&ParticleHandles, CopyContext = *CVDContextData](int32 ParticleIndex)
	{
		CVD_SCOPE_CONTEXT(CopyContext)
		TraceParticle(ParticleHandles.Handle(ParticleIndex).Get(), CopyContext);
	});
}

void FChaosVisualDebuggerTrace::TraceParticleDestroyed(const Chaos::FGeometryParticleHandle* ParticleHandle)
{
	using namespace Chaos::VisualDebugger::Utils;
	if (!IsTracing())
	{
		return;
	}

	if (!ParticleHandle)
	{
		UE_LOGF(LogChaos, Warning, "Tried to Trace a null particle %s", __FUNCTION__);
		return;
	}

	SessionState.TracedGeometryHashCache.RemoveCachedTraceObjectID(ParticleHandle->GetGeometry());
	
	const FChaosVDContext* CVDContextData = FChaosVDThreadContext::Get().GetCurrentContext(EChaosVDContextType::Solver);
	if (!IsContextEnabledAndValid(CVDContextData))
	{
		return;
	}

	int32 ParticleID = ParticleHandle->UniqueIdx().Idx;
	int32 SolverID = CVDContextData->Id;
	uint64 Cycle = FPlatformTime::Cycles64();

	UE_AUTORTFM_ONCOMMIT(SolverID, Cycle, ParticleID)
	{
		UE_TRACE_MINIMAL_LOG(ChaosVDLogger, ChaosVDParticleDestroyed, ChaosVDChannel)
			<< ChaosVDParticleDestroyed.SolverID(SolverID)
			<< ChaosVDParticleDestroyed.Cycle(Cycle)
			<< ChaosVDParticleDestroyed.ParticleID(ParticleID);
	};
}

bool FChaosVisualDebuggerTrace::IsInRelevancyVolume(const Chaos::FGeometryParticleHandle* InParticleHandle)
{
	if (!InParticleHandle)
	{
		return false;
	}

	if (!SessionState.TraceRelevancyVolume.IsValid)
	{
		return true;
	}

	const Chaos::FImplicitObjectRef ParticleGeometry = InParticleHandle->GetGeometry();
	return ParticleGeometry && ParticleGeometry->HasBoundingBox() ? IsInRelevancyVolume(ParticleGeometry->CalculateTransformedBounds(InParticleHandle->GetTransformXR()))
										: IsInRelevancyVolume(InParticleHandle->GetX());
}

bool FChaosVisualDebuggerTrace::IsInRelevancyVolume(const Chaos::FImplicitObject* Geometry, const Chaos::FRigidTransform3& InTransform)
{
	if (!Geometry)
	{
		return false;
	}

	if (!SessionState.TraceRelevancyVolume.IsValid)
	{
		return true;
	}

	return Geometry->HasBoundingBox() ? IsInRelevancyVolume(Geometry->CalculateTransformedBounds(InTransform)) : IsInRelevancyVolume(InTransform.GetLocation());
}

void FChaosVisualDebuggerTrace::TraceParticleClusterChildData(const Chaos::TParticleView<Chaos::TPBDRigidParticles<Chaos::FReal, 3>>& ParticlesView, Chaos::FRigidClustering* ClusteringData, const FChaosVDContext& CVDContextData)
{
	if (!IsTracing())
	{
		return;
	}

	if (!ClusteringData)
	{
		return;
	}

	if (!CVDDC_ClusterParticlesChildData->IsChannelEnabled())
	{
		return;
	}

	ParticlesView.ParallelFor([CopyContext = CVDContextData, ClusteringData](auto& Particle, int32 Index)
	{
		if (Chaos::FPBDRigidClusteredParticleHandle* ClusteredParticle = Particle.Handle()->CastToClustered())
		{
			CVD_SCOPE_CONTEXT(CopyContext);
			if (const TArray<Chaos::FPBDRigidParticleHandle*>* ChildrenHandles = ClusteringData->GetChildrenMap().Find(ClusteredParticle))
			{
				const TArray<Chaos::FPBDRigidParticleHandle*>& ChildrenHandlesArray = *ChildrenHandles;
				for (Chaos::FPBDRigidParticleHandle* ParticleHandle : ChildrenHandlesArray)
				{
					TraceParticle(ParticleHandle);
				}
			}
		}
	});
}

void FChaosVisualDebuggerTrace::TraceParticlesSoA(const Chaos::FPBDRigidsSOAs& ParticlesSoA, Chaos::FRigidClustering* ClusteringData)
{
	using namespace Chaos::VisualDebugger::Utils;
	if (!IsTracing())
	{
		return;
	}

	const FChaosVDContext* CVDContextData = FChaosVDThreadContext::Get().GetCurrentContext(EChaosVDContextType::Solver);

	if (!IsContextEnabledAndValid(CVDContextData))
	{
		return;
	}

	// If this solver is not being delta recorded, Trace all the particles
	if (ShouldPerformFullCapture(CVDContextData->Id))
	{
		TraceParticlesView(ParticlesSoA.GetAllParticlesView());
		return;
	}

	TraceParticlesView(ParticlesSoA.GetDirtyParticlesView());

	// If we are recording a delta frame, we need to also record the child particles of any cluster (if we have clustering data available)
	TraceParticleClusterChildData(ParticlesSoA.GetDirtyParticlesView(), ClusteringData, *CVDContextData);
}

void FChaosVisualDebuggerTrace::TraceCamera(AActor* Object, UWorld* World, FName CameraName, const FVector& Position, const FQuat& Rotation, float FOV)
{
	using namespace Chaos::VisualDebugger::Utils;
	if (!Object || !IsTracing() || !CVDDC_CameraPaths->IsChannelEnabled())
	{
		return;
	}

	FChaosVDCameraDataWrapper DataWrapper;

	DataWrapper.Camera.CameraName = CameraName;
	DataWrapper.Camera.ActorName = Object->GetFName();
	DataWrapper.Camera.ActorAssetPath = FSoftObjectPath(Object->GetClass()).GetAssetPath();
	DataWrapper.Camera.MapAssetPath = FTopLevelAssetPath(World);
	DataWrapper.Position = Position;
	DataWrapper.Rotation = Rotation;
	DataWrapper.FOV = FOV;

	FChaosVDScopedTLSBufferAccessor TLSDataBuffer;
	Chaos::VisualDebugger::WriteDataToBuffer(TLSDataBuffer.BufferRef, DataWrapper);

	TraceBinaryData(TLSDataBuffer.BufferRef, FChaosVDCameraDataWrapper::WrapperTypeName);
}

void FChaosVisualDebuggerTrace::SetupForFullCaptureIfNeeded(int32 SolverID, bool& bOutFullCaptureRequested)
{
	SessionState.DeltaRecordingStatesLock.ReadLock();
	bOutFullCaptureRequested = SessionState.RequestedFullCaptureSolverIDs.Contains(SolverID) || !SessionState.SolverIDsForDeltaRecording.Contains(SolverID);
	SessionState.DeltaRecordingStatesLock.ReadUnlock();

	if (bOutFullCaptureRequested)
	{
		UE::TWriteScopeLock WriteLock(SessionState.DeltaRecordingStatesLock);
		SessionState.SolverIDsForDeltaRecording.Remove(SolverID);
		SessionState.RequestedFullCaptureSolverIDs.Remove(SolverID);
	}
}

int32 FChaosVisualDebuggerTrace::GetSolverID(Chaos::FPhysicsSolverBase& Solver)
{
	return Solver.GetChaosVDContextData().Id;
}

bool FChaosVisualDebuggerTrace::ShouldPerformFullCapture(int32 SolverID)
{
	UE::TReadScopeLock ReadLock(SessionState.DeltaRecordingStatesLock);
	int32* FoundSolverID = SessionState.SolverIDsForDeltaRecording.Find(SolverID);

	// If the solver ID is on the SolverIDsForDeltaRecording set, it means we should NOT perform a full capture
	return FoundSolverID == nullptr;
}

void FChaosVisualDebuggerTrace::TraceMidPhase(const Chaos::FParticlePairMidPhase* MidPhase)
{
	using namespace Chaos::VisualDebugger::Utils;
	if (!IsTracing())
	{
		return;
	}

	const FChaosVDContext* CVDContextData = FChaosVDThreadContext::Get().GetCurrentContext(EChaosVDContextType::Solver);

	if (!IsContextEnabledAndValid(CVDContextData))
	{
		return;
	}

	if (!MidPhase->IsValid())
	{
		return;
	}

	using namespace Chaos;
	const bool bIsInRelevancyVolume = IsInRelevancyVolume(MidPhase->GetParticle0()) || IsInRelevancyVolume(MidPhase->GetParticle1());
	if (!bIsInRelevancyVolume)
	{
		return;
	}

	FChaosVDParticlePairMidPhase CVDMidPhase = FChaosVDDataWrapperUtils::BuildMidPhaseDataWrapperFromMidPhase(*MidPhase);
	CVDMidPhase.SolverID = CVDContextData->Id;

	FChaosVDScopedTLSBufferAccessor TLSDataBuffer;
	Chaos::VisualDebugger::WriteDataToBuffer(TLSDataBuffer.BufferRef, CVDMidPhase);

	TraceBinaryData(TLSDataBuffer.BufferRef, FChaosVDParticlePairMidPhase::WrapperTypeName);
}

void FChaosVisualDebuggerTrace::TraceMidPhasesFromCollisionConstraints(Chaos::FPBDCollisionConstraints& InCollisionConstraints)
{
	using namespace Chaos::VisualDebugger::Utils;
	if (!IsTracing())
	{
		return;
	}

	const FChaosVDContext* CVDContextData = FChaosVDThreadContext::Get().GetCurrentContext(EChaosVDContextType::Solver);

	if (!IsContextEnabledAndValid(CVDContextData))
	{
		return;
	}

	InCollisionConstraints.GetConstraintAllocator().VisitMidPhases([CopyContext = *CVDContextData](const Chaos::FParticlePairMidPhase& MidPhase) -> Chaos::ECollisionVisitorResult
	{
		CVD_SCOPE_CONTEXT(CopyContext)
		CVD_TRACE_MID_PHASE(&MidPhase);
		return Chaos::ECollisionVisitorResult::Continue;
	});
}

void FChaosVisualDebuggerTrace::TraceJointsConstraints(Chaos::FPBDJointConstraints& InJointConstraints)
{
	using namespace Chaos::VisualDebugger::Utils;

	if (!IsTracing())
	{
		return;
	}

	const FChaosVDContext* CVDContextData = FChaosVDThreadContext::Get().GetCurrentContext(EChaosVDContextType::Solver);

	if (!IsContextEnabledAndValid(CVDContextData))
	{
		return;
	}

	using namespace Chaos;
	const FPBDJointConstraints::FHandles& JointHandles = InJointConstraints.GetConstConstraintHandles();

	ParallelFor(JointHandles.Num(), [&JointHandles, CopyContext = *CVDContextData](int32 ConstraintIndex)
	{
		CVD_SCOPE_CONTEXT(CopyContext);

		if (FPBDJointConstraints::FConstraintContainerHandle* ConstraintHandle = JointHandles[ConstraintIndex])
		{
			FParticlePair ParticlePair = ConstraintHandle->GetConstrainedParticles();
			bool bIsInRelevancyVolume = IsInRelevancyVolume(ParticlePair[0]) || IsInRelevancyVolume(ParticlePair[1]);
			if (!bIsInRelevancyVolume)
			{
				return;
			}

			FChaosVDJointConstraint WrappedJointConstraintData = FChaosVDDataWrapperUtils::BuildJointDataWrapper(ConstraintHandle);
			
			WrappedJointConstraintData.SolverID = CopyContext.Id;
			
			FChaosVDScopedTLSBufferAccessor TLSDataBuffer;
			VisualDebugger::WriteDataToBuffer(TLSDataBuffer.BufferRef, WrappedJointConstraintData);
			
			TraceBinaryData(TLSDataBuffer.BufferRef, FChaosVDJointConstraint::WrapperTypeName);
		}
	});
}

void FChaosVisualDebuggerTrace::TraceCharacterGroundConstraints(Chaos::FCharacterGroundConstraintContainer& InConstraints)
{
	using namespace Chaos::VisualDebugger::Utils;

	if (!IsTracing())
	{
		return;
	}

	const FChaosVDContext* CVDContextData = FChaosVDThreadContext::Get().GetCurrentContext(EChaosVDContextType::Solver);

	if (!IsContextEnabledAndValid(CVDContextData))
	{
		return;
	}

	using namespace Chaos;
	const FCharacterGroundConstraintContainer::FConstConstraints& ConstraintHandles = InConstraints.GetConstConstraints();

	ParallelFor(ConstraintHandles.Num(), [&ConstraintHandles, CopyContext = *CVDContextData](int32 ConstraintIndex)
	{
		CVD_SCOPE_CONTEXT(CopyContext);

		if (const FCharacterGroundConstraintHandle* const GroundConstraint = ConstraintHandles[ConstraintIndex])
		{
			FParticlePair ParticlePair = GroundConstraint->GetConstrainedParticles();
			const bool bIsInRelevancyVolume = IsInRelevancyVolume(ParticlePair[0]) || IsInRelevancyVolume(ParticlePair[1]);
			if (!bIsInRelevancyVolume)
			{
				return;
			}

			FChaosVDCharacterGroundConstraint WrappedConstraintData = FChaosVDDataWrapperUtils::BuildCharacterGroundConstraintDataWrapper(GroundConstraint);

			WrappedConstraintData.SolverID = CopyContext.Id;

			FChaosVDScopedTLSBufferAccessor TLSDataBuffer;
			VisualDebugger::WriteDataToBuffer(TLSDataBuffer.BufferRef, WrappedConstraintData);

			TraceBinaryData(TLSDataBuffer.BufferRef, FChaosVDCharacterGroundConstraint::WrapperTypeName);
		}
	});
}

void FChaosVisualDebuggerTrace::TraceCollisionConstraint(const Chaos::FPBDCollisionConstraint* CollisionConstraint)
{
	using namespace Chaos::VisualDebugger::Utils;
	if (!IsTracing())
	{
		return;
	}

	const FChaosVDContext* CVDContextData = FChaosVDThreadContext::Get().GetCurrentContext(EChaosVDContextType::Solver);

	if (!IsContextEnabledAndValid(CVDContextData))
	{
		return;
	}

	using namespace Chaos;

	if (CollisionConstraint)
	{
		FParticlePair ParticlePair = CollisionConstraint->GetConstrainedParticles();
		const bool bIsInRelevancyVolume = IsInRelevancyVolume(ParticlePair[0]) || IsInRelevancyVolume(ParticlePair[1]);
		if (!bIsInRelevancyVolume)
		{
			return;
		}

		FChaosVDConstraint CVDConstraint = FChaosVDDataWrapperUtils::BuildConstraintDataWrapperFromConstraint(*CollisionConstraint);
		CVDConstraint.SolverID = CVDContextData->Id;

		FChaosVDScopedTLSBufferAccessor TLSDataBuffer;
		VisualDebugger::WriteDataToBuffer(TLSDataBuffer.BufferRef, CVDConstraint);

		TraceBinaryData(TLSDataBuffer.BufferRef, FChaosVDConstraint::WrapperTypeName);
	}
}

void FChaosVisualDebuggerTrace::TraceCollisionConstraintView(TArrayView<Chaos::FPBDCollisionConstraint* const> CollisionConstraintView)
{
	using namespace Chaos::VisualDebugger::Utils;

	if (!IsTracing())
	{
		return;
	}

	const FChaosVDContext* CVDContextData = FChaosVDThreadContext::Get().GetCurrentContext(EChaosVDContextType::Solver);

	if (!IsContextEnabledAndValid(CVDContextData))
	{
		return;
	}

	ParallelFor(CollisionConstraintView.Num(), [&CollisionConstraintView, CopyContext = *CVDContextData](int32 ConstraintIndex)
	{
		CVD_SCOPE_CONTEXT(CopyContext);
		TraceCollisionConstraint(CollisionConstraintView[ConstraintIndex]);
	});
}

void FChaosVisualDebuggerTrace::TraceConstraintsContainer(TConstArrayView<Chaos::FPBDConstraintContainer*> ConstraintContainersView)
{
	if (!IsTracing())
	{
		return;
	}

	for (Chaos::FPBDConstraintContainer* ConstraintContainer : ConstraintContainersView)
	{
		if (ConstraintContainer)
		{
			if (ConstraintContainer->GetConstraintHandleType().IsA(Chaos::FPBDJointConstraintHandle::StaticType()))
			{
				Chaos::FPBDJointConstraints* const JointConstraintPtr = static_cast<Chaos::FPBDJointConstraints*>(ConstraintContainer);
				if (JointConstraintPtr->GetUseLinearSolver())
				{
					CVD_TRACE_JOINT_CONSTRAINTS(CVDDC_JointLinearConstraints, *JointConstraintPtr);
				}
				else
				{
					CVD_TRACE_JOINT_CONSTRAINTS(CVDDC_JointNonLinearConstraints, *JointConstraintPtr);
				}
				
			}
			else if (ConstraintContainer->GetConstraintHandleType().IsA(Chaos::FPBDCollisionConstraint::StaticType()))
			{
				CVD_TRACE_STEP_MID_PHASES_FROM_COLLISION_CONSTRAINTS(CVDDC_EndOfEvolutionCollisionConstraints, *static_cast<Chaos::FPBDCollisionConstraints*>(ConstraintContainer));
			}
			else if (ConstraintContainer->GetConstraintHandleType().IsA(Chaos::FCharacterGroundConstraintHandle::StaticType()))
			{
				CVD_TRACE_CHARACTER_GROUND_CONSTRAINTS(CVDDC_CharacterGroundConstraints, *static_cast<Chaos::FCharacterGroundConstraintContainer*>(ConstraintContainer));
			}
		}
	}
}

void FChaosVisualDebuggerTrace::TraceSolverFrameStart(const FChaosVDContext& ContextData, const FString& InDebugName, int32 FrameNumber)
{
	if (!IsTracing())
	{
		return;
	}

	if (!ensure(ContextData.Id != INDEX_NONE))
	{
		return;
	}

	if (!ensure(ContextData.Type == static_cast<int32>(EChaosVDContextType::Solver)))
	{
		return;
	}

	FChaosVDThreadContext::Get().PushContext(ContextData);

	bool bIsReSimulatedFrame = EnumHasAnyFlags(static_cast<EChaosVDContextAttributes>(ContextData.Attributes), EChaosVDContextAttributes::Resimulated);

	// Check if we need to do a full capture for this solver, and setup accordingly
	bool bOutIsFullCaptureRequested;
	SetupForFullCaptureIfNeeded(ContextData.Id, bOutIsFullCaptureRequested);

	UE_AUTORTFM_OPEN
	{
		UE_TRACE_MINIMAL_LOG(ChaosVDLogger, ChaosVDSolverFrameStart, ChaosVDChannel)
			<< ChaosVDSolverFrameStart.SolverID(ContextData.Id)
			<< ChaosVDSolverFrameStart.Cycle(FPlatformTime::Cycles64())
			<< ChaosVDSolverFrameStart.DebugName(*InDebugName, InDebugName.Len())
			<< ChaosVDSolverFrameStart.IsKeyFrame(bOutIsFullCaptureRequested)
			<< ChaosVDSolverFrameStart.IsReSimulated(bIsReSimulatedFrame)
			<< ChaosVDSolverFrameStart.CurrentFrameNumber(FrameNumber);
	};
}

void FChaosVisualDebuggerTrace::TraceSolverFrameEnd(const FChaosVDContext& ContextData)
{
	if (!IsTracing())
	{
		return;
	}

	FChaosVDThreadContext::Get().PopContext();

	if (!ensure(ContextData.Id != INDEX_NONE))
	{
		return;
	}

	{
		UE::TWriteScopeLock WriteLock(SessionState.DeltaRecordingStatesLock);
		if (!SessionState.SolverIDsForDeltaRecording.Contains(ContextData.Id))
		{
			SessionState.SolverIDsForDeltaRecording.Add(ContextData.Id);
		}
	}

	UE_AUTORTFM_OPEN
	{
		UE_TRACE_MINIMAL_LOG(ChaosVDLogger, ChaosVDSolverFrameEnd, ChaosVDChannel)
			<< ChaosVDSolverFrameEnd.SolverID(ContextData.Id)
			<< ChaosVDSolverFrameEnd.Cycle(FPlatformTime::Cycles64());
	};
}

void FChaosVisualDebuggerTrace::TraceSolverStepStart(FStringView StepName)
{
	using namespace Chaos::VisualDebugger::Utils;

	if (!IsTracing())
	{
		return;
	}

	const FChaosVDContext* CVDContextData = FChaosVDThreadContext::Get().GetCurrentContext(EChaosVDContextType::Solver);
	if (!IsContextEnabledAndValid(CVDContextData))
	{
		return;
	}

	UE_AUTORTFM_OPEN
	{
		UE_TRACE_MINIMAL_LOG(ChaosVDLogger, ChaosVDSolverStepStart, ChaosVDChannel)
			<< ChaosVDSolverStepStart.Cycle(FPlatformTime::Cycles64())
			<< ChaosVDSolverStepStart.SolverID(CVDContextData->Id)
			<< ChaosVDSolverStepStart.StepName(StepName.GetData(), GetNum(StepName));
	};
}

void FChaosVisualDebuggerTrace::TraceSolverStepEnd()
{
	using namespace Chaos::VisualDebugger::Utils;

	if (!IsTracing())
	{
		return;
	}

	const FChaosVDContext* CVDContextData = FChaosVDThreadContext::Get().GetCurrentContext(EChaosVDContextType::Solver);
	if (!IsContextEnabledAndValid(CVDContextData))
	{
		return;
	}

	UE_AUTORTFM_OPEN
	{
		UE_TRACE_MINIMAL_LOG(ChaosVDLogger, ChaosVDSolverStepEnd, ChaosVDChannel)
			<< ChaosVDSolverStepEnd.Cycle(FPlatformTime::Cycles64())
			<< ChaosVDSolverStepEnd.SolverID(CVDContextData->Id);
	};
}

void FChaosVisualDebuggerTrace::TraceSolverSimulationSpace(const Chaos::FRigidTransform3& Transform)
{
	using namespace Chaos::VisualDebugger::Utils;

	if (!IsTracing())
	{
		return;
	}

	const FChaosVDContext* CVDContextData = FChaosVDThreadContext::Get().GetCurrentContext(EChaosVDContextType::Solver);
	if (!IsContextEnabledAndValid(CVDContextData))
	{
		return;
	}

	UE_AUTORTFM_OPEN
	{
		UE_TRACE_MINIMAL_LOG(ChaosVDLogger, ChaosVDSolverSimulationSpace, ChaosVDChannel)
			<< ChaosVDSolverSimulationSpace.Cycle(FPlatformTime::Cycles64())
			<< ChaosVDSolverSimulationSpace.SolverID(CVDContextData->Id)
			<< CVD_TRACE_VECTOR_ON_EVENT(ChaosVDSolverSimulationSpace, Position, Transform.GetLocation())
			<< CVD_TRACE_ROTATOR_ON_EVENT(ChaosVDSolverSimulationSpace, Rotation, Transform.GetRotation());
	};
}

void FChaosVisualDebuggerTrace::TraceBinaryData(TConstArrayView<uint8> InData, FStringView TypeName, EChaosVDTraceBinaryDataOptions Options)
{
	if (!IsTracing() && !EnumHasAnyFlags(Options, EChaosVDTraceBinaryDataOptions::ForceTrace))
	{
		return;
	}

	//TODO: This might overflow
	static std::atomic<int32> LastDataID;
	
	int32 DataID = INDEX_NONE;
	
	UE_AUTORTFM_OPEN
	{
		DataID = LastDataID++;

		ensure(DataID < TNumericLimits<int32>::Max());

		TConstArrayView<uint8> DataViewToTrace = InData;

		// Handle Compression if enabled
		const bool bIsCompressed = Chaos::VisualDebugger::Cvars::bCompressBinaryData;
		TArray<uint8> CompressedData;
		if (bIsCompressed)
		{
			CompressedData.Reserve(CompressedData.Num());
			FOodleCompressedArray::CompressData(CompressedData, InData.GetData(),InData.Num(), FOodleDataCompression::ECompressor::Kraken,
				static_cast<FOodleDataCompression::ECompressionLevel>(Chaos::VisualDebugger::Cvars::CompressionMode));

			DataViewToTrace = CompressedData;
		}

		const uint32 DataSize = static_cast<uint32>(DataViewToTrace.Num());
		constexpr uint32 MaxChunkSize = TNumericLimits<uint16>::Max();
		const uint32 ChunkNum = (DataSize + MaxChunkSize - 1) / MaxChunkSize;

		UE_TRACE_MINIMAL_LOG(ChaosVDLogger, ChaosVDBinaryDataStart, ChaosVDChannel)
			<< ChaosVDBinaryDataStart.Cycle(FPlatformTime::Cycles64())
			<< ChaosVDBinaryDataStart.TypeName(TypeName.GetData(), TypeName.Len())
			<< ChaosVDBinaryDataStart.DataID(DataID)
			<< ChaosVDBinaryDataStart.DataSize(DataSize)
			<< ChaosVDBinaryDataStart.OriginalSize(InData.Num())
			<< ChaosVDBinaryDataStart.IsCompressed(bIsCompressed);

		uint32 RemainingSize = DataSize;
		for (uint32 Index = 0; Index < ChunkNum; ++Index)
		{
			const uint16 Size = static_cast<uint16>(FMath::Min(RemainingSize, MaxChunkSize));
			const uint8* ChunkData = DataViewToTrace.GetData() + MaxChunkSize * Index;

			UE_TRACE_MINIMAL_LOG(ChaosVDLogger, ChaosVDBinaryDataContent, ChaosVDChannel)
				<< ChaosVDBinaryDataContent.Cycle(FPlatformTime::Cycles64())
				<< ChaosVDBinaryDataContent.DataID(DataID)
				<< ChaosVDBinaryDataContent.RawData(ChunkData, Size);

			RemainingSize -= Size;
		}

		ensure(RemainingSize == 0);
	};
	
	// Note: AutoRTFM is only partially supported at the moment.
	// The approach taken here is that we trace serialized data regardless if the transaction will fail or not (to avoid allocating a new buffer and copying the data)
	// but we only commit the last trace event that tells the CVD editor that the data is ready to be processed if the transaction succeeds (ChaosVDBinaryDataEnd). This allows us to ensure we don't load rolled back data automatically.
	// In non-transacted callstacks this will be executed immediately (therefore the behaviour is the same as usual), but in transacted callstacks the commit will be done after the transaction completes in a call done from the game thread.
	// This might pose an issue for data recorded outside the Game Thread as the assumption CVD relies on (all data is loaded in the exact order as it was recorded relative to other trace events) will not be valid.
	// This means we might trace the ChaosVDBinaryDataEnd event when the [ Frame / Solver Stage ] end event to which the data belong, was already issued and in consequence CVD will load the data in the incorrect frame
	// Currently the only calls done within a transaction should only be scene queries, and as they are done from the Game Thread, the framing / timing during load of the CVD recording should still be correct.
	
	UE_AUTORTFM_ONCOMMIT(DataID)
	{
		UE_TRACE_MINIMAL_LOG(ChaosVDLogger, ChaosVDBinaryDataEnd, ChaosVDChannel)
			<< ChaosVDBinaryDataEnd.Cycle(FPlatformTime::Cycles64())
			<< ChaosVDBinaryDataEnd.DataID(DataID);
	};

	UE_AUTORTFM_ONABORT(DataID)
	{
		UE_TRACE_MINIMAL_LOG(ChaosVDLogger, ChaosVDRolledBackDataID, ChaosVDChannel)
			<< ChaosVDRolledBackDataID.DataID(DataID);
	};
}

void FChaosVisualDebuggerTrace::TraceImplicitObject(FChaosVDImplicitObjectWrapper WrappedGeometryData)
{
	if (!IsTracing())
	{
		return;
	}

	ensureMsgf(false, TEXT("This method is deprecated and should not be used - The geometry provided will not be traced"));
}

uint32 FChaosVisualDebuggerTrace::TraceImplicitObject(Chaos::FImplicitObject* GeometryPtr)
{
	if (!IsTracing())
	{
		return 0;
	}

	if (!GeometryPtr)
	{
		return 0;
	}

	bool bGeometryAlreadyTraced = false;
	const uint32 GeometryHash = SessionState.TracedGeometryHashCache.FindOrProduceTraceObjectID(GeometryPtr, [GeometryPtr](){ return GeometryPtr->GetTypeHash(); }, bGeometryAlreadyTraced);

	/** If the geometry hash was not cached already, it means it is the first time we see this geometry and therefore we need to serialize */
	if (bGeometryAlreadyTraced)
	{
		return GeometryHash;
	}

	FChaosVDImplicitObjectWrapper GeometryDataWrapper;
	GeometryDataWrapper.Hash = GeometryHash;
	GeometryDataWrapper.ImplicitObject = GeometryPtr;

	FChaosVDScopedTLSBufferAccessor TLSDataBuffer;

	if (SessionState.UsingSharedGeometryArchiveContext)
	{
		// When we use the shared archive, geometry serialization happens essentially in serial
		// And we need to make sure we unlock the context only after the geometry data was written to the trace stream, otherwise
		// during deserialization we might try to deserialize it before it is loaded. The reason is that if we read an object tag in the context but we don't have the object
		// Chaos Archive context assumes the data that follows is the serialized object.

		// The perf impact of doing this serially is mitigated by serializing way less data (as everything that is instanced is serialized only once)
		// and by the fact whe only attempt to serialize a ptr once. Every subsequent attempt earlies out as this geometry hash will be already cached in TracedGeometryHashCache
		// and that cache can be accessed in parallel

		UE::TScopeLock ScopeLock(SessionState.ArchiveContextLock);
		Chaos::VisualDebugger::WriteDataToBuffer<FChaosVDImplicitObjectWrapper>(TLSDataBuffer.BufferRef, GeometryDataWrapper, SessionState.ArchiveContext);
		TraceBinaryData(TLSDataBuffer.BufferRef, FChaosVDImplicitObjectWrapper::WrapperTypeName);
	}
	else
	{
		Chaos::VisualDebugger::WriteDataToBuffer<FChaosVDImplicitObjectWrapper, Chaos::FChaosArchive>(TLSDataBuffer.BufferRef, GeometryDataWrapper);
		TraceBinaryData(TLSDataBuffer.BufferRef, FChaosVDImplicitObjectWrapper::WrapperTypeName);
	}

	return GeometryHash;
}

uint64 FChaosVisualDebuggerTrace::TraceParticleMetadata(Chaos::FGeometryParticleHandle* InParticleHandle)
{
	if (!IsTracing())
	{
		return Chaos::VD::FRecordingSessionState::InvalidUniqueID;
	}

	if (!InParticleHandle)
	{
		return Chaos::VD::FRecordingSessionState::InvalidUniqueID;
	}

	// We can't Generate Metadata without a valid physics proxy
	if (!InParticleHandle->PhysicsProxy())
	{
		return Chaos::VD::FRecordingSessionState::InvalidUniqueID;
	}

	FChaosVDParticleMetadata OutParticleMetadata;
	auto ParticleMetadataProducer = [InParticleHandle, &OutParticleMetadata]()
	{
		OutParticleMetadata = Chaos::VD::Private::GenerateParticleMetadata(InParticleHandle, SessionState);
		return OutParticleMetadata.MetadataID;
	};

	bool bMetadataAlreadyTraced = false;
	const uint64 MetadataID = SessionState.ParticleMetadataIDsCache.FindOrProduceTraceObjectID(InParticleHandle, ParticleMetadataProducer, bMetadataAlreadyTraced);

	if (!bMetadataAlreadyTraced)
	{
		FChaosVDScopedTLSBufferAccessor TLSDataBuffer;
		Chaos::VisualDebugger::WriteDataToBuffer<FChaosVDParticleMetadata>(TLSDataBuffer.BufferRef, OutParticleMetadata);

		TraceBinaryData(TLSDataBuffer.BufferRef, FChaosVDParticleMetadata::WrapperTypeName);
	}

	return MetadataID;
}

void FChaosVisualDebuggerTrace::InvalidateGeometryFromCache(const Chaos::FImplicitObject* CachedGeometryToInvalidate)
{
	if (!IsTracing())
	{
		return;
	}

	{
		UE::TScopeLock ContextScopeLock(SessionState.ArchiveContextLock);

		// TODO: Should we walk the object hierarchy and remove all inner objects?
		SessionState.ArchiveContext->ObjToTag.Remove(CachedGeometryToInvalidate);
	}
	
	SessionState.TracedGeometryHashCache.RemoveCachedTraceObjectID(CachedGeometryToInvalidate);
}

void FChaosVisualDebuggerTrace::InvalidateParticleMetadataFromCache(Chaos::FGeometryParticleHandle* InParticleToInvalidate)
{
	if (!InParticleToInvalidate)
	{
		return;
	}

	SessionState.ParticleMetadataIDsCache.RemoveCachedTraceObjectID(InParticleToInvalidate);
}

void FChaosVisualDebuggerTrace::TraceNonSolverLocation(const FVector& InLocation, FStringView DebugNameID)
{
	if (!IsTracing())
	{
		return;
	}

	UE_TRACE_MINIMAL_LOG(ChaosVDLogger, ChaosVDNonSolverLocation, ChaosVDChannel)
			<< ChaosVDNonSolverLocation.Cycle(FPlatformTime::Cycles64())
			<< CVD_TRACE_VECTOR_ON_EVENT(ChaosVDNonSolverLocation, Position, InLocation)
			<< ChaosVDNonSolverLocation.DebugName(DebugNameID.GetData(), DebugNameID.Len());
}

void FChaosVisualDebuggerTrace::TraceNonSolverTransform(const FTransform& InTransform, FStringView DebugNameID)
{
	if (!IsTracing())
	{
		return;
	}

	UE_TRACE_MINIMAL_LOG(ChaosVDLogger, ChaosVDNonSolverTransform, ChaosVDChannel)
		<< ChaosVDNonSolverTransform.Cycle(FPlatformTime::Cycles64())
		<< CVD_TRACE_VECTOR_ON_EVENT(ChaosVDNonSolverTransform, Position, InTransform.GetLocation())
		<< CVD_TRACE_VECTOR_ON_EVENT(ChaosVDNonSolverTransform, Scale, InTransform.GetScale3D())
		<< CVD_TRACE_ROTATOR_ON_EVENT(ChaosVDNonSolverTransform, Rotation, InTransform.GetRotation())
		<< ChaosVDNonSolverTransform.DebugName(DebugNameID.GetData(), DebugNameID.Len());
}

void FChaosVisualDebuggerTrace::TraceSceneQueryStart(const Chaos::FImplicitObject* InputGeometry, const FQuat& GeometryOrientation,  const FVector& Start, const FVector& End, ECollisionChannel TraceChannel, FChaosVDCollisionQueryParams&& Params, FChaosVDCollisionResponseParams&& ResponseParams, FChaosVDCollisionObjectQueryParams&& ObjectParams, EChaosVDSceneQueryType QueryType, EChaosVDSceneQueryMode QueryMode, int32 SolverID, bool bIsRetry)
{
	using namespace Chaos::VisualDebugger::Utils;

	if (!IsTracing())
	{
		return;
	}

	const FChaosVDContext* CVDContextData = FChaosVDThreadContext::Get().GetCurrentContext();
	if (!IsContextEnabledAndValid(CVDContextData))
	{
		return;
	}

	const bool bIsQueryContext = CVDContextData->Type == static_cast<int32>(EChaosVDContextType::Query) ||  CVDContextData->Type == static_cast<int32>(EChaosVDContextType::SubTraceQuery);

	if (!ensure(bIsQueryContext))
	{
		return;
	}

	auto IsInRelevancyVolumeAtPosition = [InputGeometry](const FVector& InPosition, const FQuat& InOrientation)
	{
		return InputGeometry ? IsInRelevancyVolume(InputGeometry, Chaos::FRigidTransform3(InPosition, InOrientation)) : IsInRelevancyVolume(InPosition);
	};

	// In overlap queries, only the start position is valid
	const bool bEvaluateBothPositions = QueryType != EChaosVDSceneQueryType::Overlap;
	bool bIsRelevant = bEvaluateBothPositions ? (IsInRelevancyVolumeAtPosition(Start, GeometryOrientation) || IsInRelevancyVolumeAtPosition(End, GeometryOrientation))
												: IsInRelevancyVolumeAtPosition(Start, GeometryOrientation);

	if (!bIsRelevant)
	{
		// TODO: We need to find a better way to do this. Scene queries and shape visit data are traced independently, and the stitched together during load, using the following context ID.
		// therefore invalidating this id is not supposed to change is the less intrusive way to avoid tracing the shape visit data for now
		const_cast<FChaosVDContext*>(CVDContextData)->Id  = INDEX_NONE;
		return;
	}

	FChaosVDQueryDataWrapper WrappedQueryData;

	if (InputGeometry)
	{
		WrappedQueryData.InputGeometryKey = TraceImplicitObject(const_cast<Chaos::FImplicitObject*>(InputGeometry));
	}
	
	WrappedQueryData.ID = CVDContextData->Id;
	WrappedQueryData.ParentQueryID = CVDContextData->OwnerID;
	WrappedQueryData.WorldSolverID = SolverID;
	WrappedQueryData.bIsRetryQuery = bIsRetry;
	
	WrappedQueryData.GeometryOrientation = GeometryOrientation;

	WrappedQueryData.CollisionChannel = TraceChannel;
	WrappedQueryData.StartLocation = Start;
	WrappedQueryData.EndLocation = End;

	WrappedQueryData.CollisionQueryParams = MoveTemp(Params);
	WrappedQueryData.CollisionResponseParams = MoveTemp(ResponseParams);
	WrappedQueryData.CollisionObjectQueryParams = MoveTemp(ObjectParams);

	WrappedQueryData.Mode = QueryMode;
	WrappedQueryData.Type = QueryType;

	FChaosVDScopedTLSBufferAccessor TLSDataBuffer;
	Chaos::VisualDebugger::WriteDataToBuffer(TLSDataBuffer.BufferRef, WrappedQueryData);

	TraceBinaryData(TLSDataBuffer.BufferRef, FChaosVDQueryDataWrapper::WrapperTypeName);
}

void FChaosVisualDebuggerTrace::TraceSceneQueryVisit(FChaosVDQueryVisitStep&& InQueryVisitData)
{
	using namespace Chaos::VisualDebugger::Utils;

	if (!IsTracing())
	{
		return;
	}

	const FChaosVDContext* CVDContextData = FChaosVDThreadContext::Get().GetCurrentContext();
	if (!IsContextEnabledAndValid(CVDContextData))
	{
		return;
	}

	if (CVDContextData->Id == INDEX_NONE)
	{
		return;
	}

	const bool bIsQueryContext = CVDContextData->Type == static_cast<int32>(EChaosVDContextType::Query) ||  CVDContextData->Type == static_cast<int32>(EChaosVDContextType::SubTraceQuery);

	if (!ensure(bIsQueryContext))
	{
		return;
	}

	InQueryVisitData.OwningQueryID = CVDContextData->Id;

	FChaosVDScopedTLSBufferAccessor TLSDataBuffer;
	Chaos::VisualDebugger::WriteDataToBuffer(TLSDataBuffer.BufferRef, InQueryVisitData);

	TraceBinaryData(TLSDataBuffer.BufferRef, FChaosVDQueryVisitStep::WrapperTypeName);
}

void FChaosVisualDebuggerTrace::TraceSceneAccelerationStructures(const Chaos::ISpatialAccelerationCollection<Chaos::FAccelerationStructureHandle, Chaos::FReal, 3>* InAccelerationCollection)
{
	using namespace Chaos::VisualDebugger::Utils;

	if (!IsTracing())
	{
		return;
	}

	if (!InAccelerationCollection)
	{
		return;
	}
	
	const FChaosVDContext* CVDContextData = FChaosVDThreadContext::Get().GetCurrentContext();
	if (!IsContextEnabledAndValid(CVDContextData))
	{
		return;
	}

	TArray<FChaosVDAABBTreeDataWrapper> AABBTreeDataWrappers;
	FChaosVDDataWrapperUtils::BuildDataWrapperFromAABBStructure(InAccelerationCollection, CVDContextData->Id, AABBTreeDataWrappers);

	for (FChaosVDAABBTreeDataWrapper& DataWrapper : AABBTreeDataWrappers)
	{
		FChaosVDScopedTLSBufferAccessor TLSDataBuffer;
		Chaos::VisualDebugger::WriteDataToBuffer(TLSDataBuffer.BufferRef, DataWrapper);

		TraceBinaryData(TLSDataBuffer.BufferRef, FChaosVDAABBTreeDataWrapper::WrapperTypeName);
	}
}

void FChaosVisualDebuggerTrace::TraceNetworkTickOffset(int32 TickOffset, int32 SolverID)
{
	if (!IsTracing())
	{
		return;
	}

	UE_AUTORTFM_OPEN
	{
		UE_TRACE_MINIMAL_LOG(ChaosVDLogger, ChaosVDNetworkTickOffset, ChaosVDChannel)
			<< ChaosVDNetworkTickOffset.Offset(TickOffset)
			<< ChaosVDNetworkTickOffset.SolverID(SolverID);
	};
}

bool FChaosVisualDebuggerTrace::CanTraceDebugDrawShape(int32& OutSolverID)
{
	using namespace Chaos::VisualDebugger::Utils;
	
	if (const FChaosVDContext* CVDContextData = FChaosVDThreadContext::Get().GetCurrentContext(EChaosVDContextType::Solver))
	{
		if (!IsContextEnabledAndValid(CVDContextData))
		{
			return false;
		}
	
		OutSolverID = OutSolverID == INDEX_NONE ? CVDContextData->Id : OutSolverID;

		return true;
	}
	return true;
}

void FChaosVisualDebuggerTrace::TraceDebugDrawBox(const FBox& InBox, FName Tag, FColor Color, int32 SolverID)
{
	if (!IsTracing())
	{
		return;
	}

	// Generic Debug Draw might not have a context if they are recorded from the game thread in game code, in that case we allow the trace anyway
	if (!CanTraceDebugDrawShape(SolverID))
	{
		return;
	}

	if (!IsInRelevancyVolume(InBox))
	{
		return;
	}

	FChaosVDDebugDrawBoxDataWrapper DataWrapper;
	DataWrapper.SolverID = SolverID;
	DataWrapper.Tag = Tag;
	DataWrapper.Color = Color;
	DataWrapper.Box = InBox;
	DataWrapper.ThreadContext = IsInGameThread() ? EChaosVDParticleContext::GameThread : EChaosVDParticleContext::PhysicsThread;

	DataWrapper.MarkAsValid();

	FChaosVDScopedTLSBufferAccessor TLSDataBuffer;
	Chaos::VisualDebugger::WriteDataToBuffer(TLSDataBuffer.BufferRef, DataWrapper);

	TraceBinaryData(TLSDataBuffer.BufferRef, FChaosVDDebugDrawBoxDataWrapper::WrapperTypeName);
}

void FChaosVisualDebuggerTrace::TraceDebugDrawLine(const FVector& InStartLocation, const FVector& InEndLocation, FName Tag, FColor Color, int32 SolverID)
{
	if (!IsTracing())
	{
		return;
	}

	// Generic Debug Draw might not have a context if they are recorded from the game thread in game code, in that case we allow the trace anyway
	if (!CanTraceDebugDrawShape(SolverID))
	{
		return;
	}

	if (!IsInRelevancyVolume(InStartLocation, InEndLocation))
	{
		return;
	}

	FChaosVDDebugDrawLineDataWrapper DataWrapper;
	DataWrapper.SolverID = SolverID;
	DataWrapper.Tag = Tag;
	DataWrapper.Color = Color;
	DataWrapper.StartLocation = InStartLocation;
	DataWrapper.EndLocation = InEndLocation;
	DataWrapper.ThreadContext = IsInGameThread() ? EChaosVDParticleContext::GameThread : EChaosVDParticleContext::PhysicsThread;

	DataWrapper.MarkAsValid();

	FChaosVDScopedTLSBufferAccessor TLSDataBuffer;
	Chaos::VisualDebugger::WriteDataToBuffer(TLSDataBuffer.BufferRef, DataWrapper);

	TraceBinaryData(TLSDataBuffer.BufferRef, FChaosVDDebugDrawLineDataWrapper::WrapperTypeName);
}

void FChaosVisualDebuggerTrace::TraceDebugDrawVector(const FVector& InStartLocation, const FVector& InVector, FName Tag, FColor Color, int32 SolverID)
{
	if (!IsTracing())
	{
		return;
	}
	
	// Generic Debug Draw might not have a context if they are recorded from the game thread in game code, in that case we allow the trace anyway
	if (!CanTraceDebugDrawShape(SolverID))
	{
		return;
	}

	FVector EndLocation = InStartLocation + InVector;

	if (!IsInRelevancyVolume(InStartLocation, EndLocation))
	{
		return;
	}

	FChaosVDDebugDrawLineDataWrapper DataWrapper;
	DataWrapper.SolverID = SolverID;
	DataWrapper.Tag = Tag;
	DataWrapper.Color = Color;
	DataWrapper.StartLocation = InStartLocation;
	DataWrapper.EndLocation = EndLocation;
	DataWrapper.bIsArrow = true;
	DataWrapper.ThreadContext = IsInGameThread() ? EChaosVDParticleContext::GameThread : EChaosVDParticleContext::PhysicsThread;

	DataWrapper.MarkAsValid();

	FChaosVDScopedTLSBufferAccessor TLSDataBuffer;
	Chaos::VisualDebugger::WriteDataToBuffer(TLSDataBuffer.BufferRef, DataWrapper);

	TraceBinaryData(TLSDataBuffer.BufferRef, FChaosVDDebugDrawLineDataWrapper::WrapperTypeName);
}

void FChaosVisualDebuggerTrace::TraceDebugDrawSphere(const FVector& Center, float Radius, FName Tag, FColor Color, int32 SolverID)
{
	if (!IsTracing())
	{
		return;
	}

	// Generic Debug Draw might not have a context if they are recorded from the game thread in game code, in that case we allow the trace anyway
	if (!CanTraceDebugDrawShape(SolverID))
	{
		return;
	}

	if (!IsInRelevancyVolume(Center, Radius))
	{
		return;
	}

	FChaosVDDebugDrawSphereDataWrapper DataWrapper;
	DataWrapper.SolverID = SolverID;
	DataWrapper.Tag = Tag;
	DataWrapper.Color = Color;
	DataWrapper.Origin = Center;
	DataWrapper.Radius = Radius;
	DataWrapper.ThreadContext = IsInGameThread() ? EChaosVDParticleContext::GameThread : EChaosVDParticleContext::PhysicsThread;

	DataWrapper.MarkAsValid();

	FChaosVDScopedTLSBufferAccessor TLSDataBuffer;
	Chaos::VisualDebugger::WriteDataToBuffer(TLSDataBuffer.BufferRef, DataWrapper);

	TraceBinaryData(TLSDataBuffer.BufferRef, FChaosVDDebugDrawSphereDataWrapper::WrapperTypeName);
}

void FChaosVisualDebuggerTrace::TraceDebugDrawImplicitObject(const Chaos::FImplicitObject* Implicit, const FTransform& InParentTransform, FName Tag, FColor Color, int32 SolverID)
{
	if (!IsTracing())
	{
		return;
	}

	if (!Implicit)
	{
		return;
	}
	
	// Generic Debug Draw might not have a context if they are recorded from the game thread in game code, in that case we allow the trace anyway
	if (!CanTraceDebugDrawShape(SolverID))
	{
		return;
	}

	if (SessionState.TraceRelevancyVolume.IsValid)
	{
		if (Implicit->HasBoundingBox() && !IsInRelevancyVolume(Implicit->CalculateTransformedBounds(InParentTransform)))
		{
			return;
		}
	}

	FChaosVDDebugDrawImplicitObjectDataWrapper DataWrapper;
	DataWrapper.SolverID = SolverID;
	DataWrapper.Tag = Tag;
	DataWrapper.Color = Color;
	DataWrapper.ParentTransform = InParentTransform;
	DataWrapper.ThreadContext = IsInGameThread() ? EChaosVDParticleContext::GameThread : EChaosVDParticleContext::PhysicsThread;

	DataWrapper.ImplicitObjectHash = TraceImplicitObject(const_cast<Chaos::FImplicitObject*>(Implicit));

	DataWrapper.MarkAsValid();

	FChaosVDScopedTLSBufferAccessor TLSDataBuffer;
	Chaos::VisualDebugger::WriteDataToBuffer(TLSDataBuffer.BufferRef, DataWrapper);

	TraceBinaryData(TLSDataBuffer.BufferRef, FChaosVDDebugDrawImplicitObjectDataWrapper::WrapperTypeName);
}

void FChaosVisualDebuggerTrace::SetTraceRelevancyVolume(const FBox& InTraceRelevancyVolume)
{
	SessionState.TraceRelevancyVolume = InTraceRelevancyVolume;

	UE_TRACE_MINIMAL_LOG(ChaosVDLogger, ChaosVDTraceRelevancyVolume, ChaosVDChannel)
		<< CVD_TRACE_VECTOR_ON_EVENT(ChaosVDTraceRelevancyVolume, BoxMin, InTraceRelevancyVolume.Min)
		<< CVD_TRACE_VECTOR_ON_EVENT(ChaosVDTraceRelevancyVolume, BoxMax, InTraceRelevancyVolume.Max);
}

bool FChaosVisualDebuggerTrace::IsTracing()
{
	return SessionState.bIsTracing;
}

void FChaosVisualDebuggerTrace::RegisterEventHandlers()
{
	{
		using namespace UE::TraceBasedDebuggers;
		UE::TWriteScopeLock WriteLock(SessionState.DeltaRecordingStatesLock);

		
		if (!SessionState.RecordingStartedDelegateHandle.IsValid())
		{
			SessionState.RecordingStartedDelegateHandle = FChaosVDRuntimeModule::RegisterRecordingStartedCallback(FRecordingStateChangedDelegate::FDelegate::CreateStatic(&FChaosVisualDebuggerTrace::HandleRecordingStart));
		}

		if (!SessionState.RecordingStoppedDelegateHandle.IsValid())
		{
			SessionState.RecordingStoppedDelegateHandle = FChaosVDRuntimeModule::RegisterRecordingStopCallback(FRecordingStateChangedDelegate::FDelegate::CreateStatic(&FChaosVisualDebuggerTrace::HandleRecordingStop));
		}

		if (!SessionState.RecordingFullCaptureRequestedHandle.IsValid())
		{
			SessionState.RecordingFullCaptureRequestedHandle = FChaosVDRuntimeModule::RegisterFullCaptureRequestedCallback(FChaosVDCaptureRequestDelegate::FDelegate::CreateStatic(&FChaosVisualDebuggerTrace::PerformFullCapture));
		}
	}
}

void FChaosVisualDebuggerTrace::UnregisterEventHandlers()
{
	UE::TWriteScopeLock WriteLock(SessionState.DeltaRecordingStatesLock);
	if (SessionState.RecordingStartedDelegateHandle.IsValid())
	{
		FChaosVDRuntimeModule::RemoveRecordingStartedCallback(SessionState.RecordingStartedDelegateHandle);
	}

	if (SessionState.RecordingStoppedDelegateHandle.IsValid())
	{
		FChaosVDRuntimeModule::RemoveRecordingStopCallback(SessionState.RecordingStoppedDelegateHandle);
	}

	if (SessionState.RecordingFullCaptureRequestedHandle.IsValid())
	{
		FChaosVDRuntimeModule::RemoveFullCaptureRequestedCallback(SessionState.RecordingFullCaptureRequestedHandle);
	}

	SessionState.bIsTracing = false;
}

TSharedRef<FChaosVDSerializableNameTable>& FChaosVisualDebuggerTrace::GetNameTableInstance()
{
	return SessionState.CVDNameTable;
}

void FChaosVisualDebuggerTrace::Reset()
{
	SessionState.CVDNameTable->ResetTable();

	SessionState.TraceRelevancyVolume = FBox(EForceInit::ForceInitToZero);

	{
		UE::TWriteScopeLock WriteLock(SessionState.DeltaRecordingStatesLock);
		SessionState.RequestedFullCaptureSolverIDs.Reset();
		SessionState.SolverIDsForDeltaRecording.Reset();
	}

	SessionState.TracedGeometryHashCache.Reset();
	SessionState.ParticleMetadataIDsCache.Reset();
	SessionState.ArchiveContext = MakeShared<Chaos::FChaosArchiveContext>();
	SessionState.UsingSharedGeometryArchiveContext = Chaos::VisualDebugger::Cvars::bUsesSharedGeometryArchiveContext;
}

void FChaosVisualDebuggerTrace::HandleRecordingStop()
{
	SessionState.bIsTracing = false;
	Reset();
}

void FChaosVisualDebuggerTrace::TraceArchiveHeader()
{
	using namespace Chaos::VisualDebugger;

	TArray<uint8> HeaderDataBuffer;

	FMemoryWriter MemWriterAr(HeaderDataBuffer);

	FChaosVDArchiveHeader::Current().Serialize(MemWriterAr);

	// We intentionally trace the header when the recording start was requested but we are not in a tracing state
	// So we need to force a trace
	// We do this to ensure the header is traced before any other binary data is generated, as we will need it to be read first on load 
	TraceBinaryData(HeaderDataBuffer, FChaosVDArchiveHeader::WrapperTypeName, EChaosVDTraceBinaryDataOptions::ForceTrace);
}

void FChaosVisualDebuggerTrace::OverrideDefaultEnabledDataChannels(TConstArrayView<FString> EnabledDataChannelsOverrideList)
{
	UE_LOGF(LogChaos, Log, "[%ls] Channel list override provided - Enabling [%d] Requested channels...", ANSI_TO_TCHAR(__FUNCTION__), EnabledDataChannelsOverrideList.Num());

	using namespace Chaos::VisualDebugger;
	FChaosVDDataChannelsManager::Get().EnumerateDataChannels([&EnabledDataChannelsOverrideList](const TWeakPtr<FChaosVDOptionalDataChannel>& Channel)
	{
		if (TSharedPtr<FChaosVDOptionalDataChannel> LockedChannel = Channel.Pin())
		{
			if (LockedChannel->CanChangeEnabledState())
			{
				// This is far from efficient, but this will be called once when the recording start command is executed, and we only have a handful of channels
				const FString ChannelIdAsString = LockedChannel->GetId().ToString();
				const bool bChannelShouldBeEnabled = EnabledDataChannelsOverrideList.Contains(ChannelIdAsString);
				LockedChannel->SetChannelEnabled(bChannelShouldBeEnabled);

				UE_LOGF(LogChaos, Log, "[%ls] Setting enabled state for channel [%ls] to [%ls]...", ANSI_TO_TCHAR(__FUNCTION__), *ChannelIdAsString, bChannelShouldBeEnabled ? TEXT("True") : TEXT("False"));
			}
		}
		return true;
	});
}

void FChaosVisualDebuggerTrace::RegisterExternalParticleDebugNameGenerator(const Chaos::VD::FRecordingSessionState::FParticleMetaDataGeneratorDelegate& InCallback)
{
	if (SessionState.ExternalParticleMetadataGenerator.IsBound())
	{
		UE_LOGF(LogChaos, Error, "An external debug name generator for particles is already registered. Ignoring new register attempt");
		
		return;
	}
	
	SessionState.ExternalParticleMetadataGenerator = InCallback;
}

void FChaosVisualDebuggerTrace::HandleRecordingStart()
{
	Reset();

	FString CommandlineEnabledCVDChannels;
	constexpr bool bStopOnSeparator = false;
	if (FParse::Value(FCommandLine::Get(), TEXT("CVDDataChannelsOverride="), CommandlineEnabledCVDChannels, bStopOnSeparator))
	{
		TArray<FString> ParsedChannels;
		Chaos::VisualDebugger::ParseChannelListFromCommandArgument(ParsedChannels, CommandlineEnabledCVDChannels);
		OverrideDefaultEnabledDataChannels(ParsedChannels);
	}

	TraceArchiveHeader();

	UE_TRACE_MINIMAL_LOG(ChaosVDLogger, ChaosVDUsesAutoRTFM, ChaosVDChannel)
			<< ChaosVDUsesAutoRTFM.bUsingAutoRTFM(static_cast<bool>(UE_AUTORTFM));

	UE_TRACE_MINIMAL_LOG(ChaosVDLogger, ChaosVDMiscHeaderData, ChaosVDChannel)
		<< ChaosVDMiscHeaderData.bUsesSharedGeometryContextArchive(SessionState.UsingSharedGeometryArchiveContext);

	SessionState.bIsTracing = true;
}

void FChaosVisualDebuggerTrace::PerformFullCapture(EChaosVDFullCaptureFlags CaptureOptions)
{
	if (EnumHasAnyFlags(CaptureOptions, EChaosVDFullCaptureFlags::Particles))
	{
		UE::TWriteScopeLock WriteLock(SessionState.DeltaRecordingStatesLock);
		SessionState.RequestedFullCaptureSolverIDs.Append(SessionState.SolverIDsForDeltaRecording);
	}

	if (EnumHasAnyFlags(CaptureOptions, EChaosVDFullCaptureFlags::Geometry))
	{
		SessionState.TracedGeometryHashCache.Reset();
	}
}

#endif //WITH_CHAOS_VISUAL_DEBUGGER
