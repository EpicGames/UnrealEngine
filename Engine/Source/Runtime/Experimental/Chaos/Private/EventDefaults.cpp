// Copyright Epic Games, Inc. All Rights Reserved.

#include "EventDefaults.h"
#include "EventsData.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "Chaos/PBDRigidsEvolution.h"
#include "Chaos/PBDRigidClustering.h"
#include "Chaos/PBDCollisionConstraints.h"
#include "Chaos/CollisionResolutionTypes.h"
#include "PBDRigidsSolver.h"
#include "PhysicsProxy/SkeletalMeshPhysicsProxy.h"
#include "PhysicsProxy/StaticMeshPhysicsProxy.h"
#include "PhysicsProxy/GeometryCollectionPhysicsProxy.h"
#include "PhysicsProxy/PerSolverFieldSystem.h"
#include "ChaosSolversModule.h"
#include "Chaos/CollisionFilterData.h"

namespace Chaos
{
	DECLARE_CYCLE_STAT(TEXT("EventDefaults::RegisterCollisionEvent_Filter"), STAT_Events_RegisterCollisionEvent_Filter, STATGROUP_Chaos);
	DECLARE_CYCLE_STAT(TEXT("EventDefaults::RegisterCollisionEvent_Notify"), STAT_Events_RegisterCollisionEvent_Notify, STATGROUP_Chaos);

	/**
	 * Resolves a material from a shape and a collision.
	 * Either both shapes are simple primitives, or one is simple and the other is a heightfield or trimesh.
	 * In the case of the trimesh/heightfield the contact points should have the face index of the hit face
	 * which the geometry can translate into a material index.
	 * @param InShape - The shape to resolve a material for
	 * @param InConstraint - The collision that the shape is a part of
	 * @see FTriangleMeshImplicitObject::ContactManifoldImp
	 * @see FHeightField::ContactManifoldImp
	 */
	FMaterialHandle ResolveMaterial(const FPerShapeData* InShape, const FPBDCollisionConstraint& InConstraint)
	{
		// @todo(chaos): this does not handle PerParticleMaterials so may return the wrong value when that is used

		if (InShape && (InShape->NumMaterials() > 0))
		{
			// Simple case, one material. All primitives (Box, convex, sphere etc.) should only have one material.
			// Heightfield and Trimesh can have one or more and will require data from the contacts to resolve.
			if (InShape->NumMaterials() == 1)
			{
				return InShape->GetMaterial(0);
			}
			else if (InConstraint.NumManifoldPoints() > 0)
			{
				// We only support one material per manifold (just use the first manifold point)
				const int32 ShapeFaceIndex = InConstraint.GetManifoldPoint(0).ContactPoint.FaceIndex;
				const int32 ShapeMaterialIndex = InShape->GetGeometry()->GetMaterialIndex(ShapeFaceIndex);
				if (ShapeMaterialIndex < InShape->NumMaterials())
				{
					return InShape->GetMaterial(ShapeMaterialIndex);
				}
			}
		}

		// No valid material
		return {};
	}

	void FEventDefaults::RegisterSystemEvents(FEventManager& EventManager)
	{
		RegisterCollisionEvent(EventManager);
		RegisterBreakingEvent(EventManager);
		RegisterTrailingEvent(EventManager);
		RegisterSleepingEvent(EventManager);
		RegisterRemovalEvent(EventManager);
		RegisterCrumblingEvent(EventManager);
	}

	void FEventDefaults::RegisterCollisionEvent(FEventManager& EventManager)
	{
		EventManager.template RegisterEvent<FCollisionEventData>(EEventType::Collision, []
		(const Chaos::FPBDRigidsSolver* Solver, FCollisionEventData& CollisionEventData, bool bResetData)
		{
			check(Solver);

			EnsureIsInPhysicsThreadContext();

			SCOPE_CYCLE_COUNTER(STAT_GatherCollisionEvent);

			// #todo: This isn't working - SolverActor parameters are set on a solver but it is currently a different solver that is simulating!!
			//if (!Solver->GetEventFilters()->IsCollisionEventEnabled())
			//	return;

			if (bResetData)
			{
				CollisionEventData.Reset();
			}
			CollisionEventData.CollisionData.TimeCreated = Solver->MTime;
			CollisionEventData.PhysicsProxyToCollisionIndices.TimeCreated = Solver->MTime;
			
			const auto* Evolution = Solver->GetEvolution();

			const FPBDCollisionConstraints& CollisionRule = Evolution->GetCollisionConstraints();


			const FPBDRigidParticles& Particles = Evolution->GetParticles().GetDynamicParticles();
			const TArrayCollectionArray<ClusterId>& ClusterIdsArray = Evolution->GetRigidClustering().GetClusterIdsArray();
#if TODO_REIMPLEMENT_RIGID_CLUSTERING
			const Chaos::FPBDRigidsSolver::FClusteringType::FClusterMap& ParentToChildrenMap = Evolution->GetRigidClustering().GetChildrenMap();
#endif
			const typename Chaos::FRigidClustering::FClusterMap& ParentToChildrenMap = Evolution->GetRigidClustering().GetChildrenMap();

			if(CollisionRule.NumConstraints() > 0)
			{
				// Get the number of valid constraints (AccumulatedImpulse != 0.f and Phi < 0.f) from AllConstraintsArray
				TArray<const Chaos::FPBDCollisionConstraintHandle*> ValidCollisionHandles;
				ValidCollisionHandles.SetNumUninitialized(CollisionRule.NumConstraints());
				int32 NumValidCollisions = 0;
				{
					SCOPE_CYCLE_COUNTER(STAT_Events_RegisterCollisionEvent_Filter);
					const FReal MinDeltaVelocityForHitEvents = FChaosSolversModule::GetModule()->GetSettingsProvider().GetMinDeltaVelocityForHitEvents();
					const FPBDCollisionConstraints::FConstHandles& CollisionHandles = CollisionRule.GetConstConstraintHandles();
					TArray<bool> ValidArray;
					ValidArray.SetNum(CollisionHandles.Num());
					InnerPhysicsParallelForRange(CollisionHandles.Num(), [&](int32 StartRangeIndex, int32 EndRangeIndex)
					{
						for (int32 Index = StartRangeIndex; Index < EndRangeIndex; ++Index)
						{
							ValidArray[Index] = false;

							const Chaos::FPBDCollisionConstraintHandle* ContactHandle = CollisionHandles[Index];
							if (ContactHandle == nullptr)
							{
								continue;
							}

							if (!ContactHandle->IsEnabled())
							{
								continue;
							}

							if (NumValidCollisions >= CollisionRule.NumConstraints())
							{
								break;
							}

							const FPBDCollisionConstraint& Constraint = ContactHandle->GetContact();
							
							if (ensureMsgf(!Constraint.AccumulatedImpulse.ContainsNaN(), TEXT("Constraint contains a NaN within the accumulated impulse.")))
							{

								if (ensureMsgf(FMath::IsFinite(Constraint.GetPhi()), TEXT("Constraint Phi is not finite.")))
								{

									const FPerShapeData* Shape0 = Constraint.GetShape0();
									const FPerShapeData* Shape1 = Constraint.GetShape1();

									// If we don't have a filter - allow the notify, otherwise obey the filter flag
									const bool bFilter0Notify = Shape0 ? Shape0->GetShapeFilterData().HasFlag(EFilterFlags::ContactNotify) : true;
									const bool bFilter1Notify = Shape1 ? Shape1->GetShapeFilterData().HasFlag(EFilterFlags::ContactNotify) : true;

									if (!bFilter0Notify && !bFilter1Notify)
									{
										// No need to notify - engine didn't request notifications for either shape.
										continue;
									}

									const FGeometryParticleHandle* Particle0 = Constraint.GetParticle0();
									const FGeometryParticleHandle* Particle1 = Constraint.GetParticle1();
									const FKinematicGeometryParticleHandle* Body0 = Particle0->CastToKinematicParticle();

									// presently when a rigidbody or kinematic hits static geometry then Body1 is null
									const FKinematicGeometryParticleHandle* Body1 = Particle1->CastToKinematicParticle();

									const FKinematicGeometryParticleHandle* Primary = Body0 ? Body0 : Body1;
									const FKinematicGeometryParticleHandle* Secondary = Body0 ? Body1 : Body0;

									//
									const int32 NumManifoldPoints = Constraint.GetManifoldPoints().Num();

									if (((Constraint.IsProbe() && NumManifoldPoints > 0) || !Constraint.AccumulatedImpulse.IsZero()) && Primary)
									{
										if (ensure(!Constraint.CalculateWorldContactLocation().ContainsNaN() &&
											!Constraint.CalculateWorldContactNormal().ContainsNaN()) &&
											!Primary->GetV().ContainsNaN() &&
											!Primary->GetW().ContainsNaN() &&
											(Secondary == nullptr || ((!Secondary->GetV().ContainsNaN()) && !Secondary->GetW().ContainsNaN())))
										{
											ValidArray[Index] = true;
										}
									}
								}
							}
						}
					}, Chaos::LargeBatchSize);
					for (int32 Index = 0; Index < CollisionHandles.Num(); ++Index)
					{
						if (ValidArray[Index])
						{
							const Chaos::FPBDCollisionConstraintHandle* ContactHandle = CollisionHandles[Index];
							ValidCollisionHandles[NumValidCollisions] = ContactHandle;
							NumValidCollisions++;
						}
					}
				}

				{
					SCOPE_CYCLE_COUNTER(STAT_Events_RegisterCollisionEvent_Notify);
					ValidCollisionHandles.SetNum(NumValidCollisions);
					if (ValidCollisionHandles.Num() > 0)
					{
						FCollisionDataArray DupAllCollisionsDataArray;
						DupAllCollisionsDataArray.SetNum(NumValidCollisions);
						InnerPhysicsParallelForRange(ValidCollisionHandles.Num(), [&](int32 StartRangeIndex, int32 EndRangeIndex)
						{
							for (int32 IdxCollision = StartRangeIndex; IdxCollision < EndRangeIndex; ++IdxCollision)
							{
								const FPBDCollisionConstraint& Constraint = ValidCollisionHandles[IdxCollision]->GetContact();

								const FGeometryParticleHandle* Particle0 = Constraint.GetParticle0();
								const FGeometryParticleHandle* Particle1 = Constraint.GetParticle1();

								FCollidingData Data;
								Data.SolverTime = Solver->MTime;
								Data.Location = Constraint.CalculateWorldContactLocation();
								Data.AccumulatedImpulse = Constraint.AccumulatedImpulse;
								Data.Normal = Constraint.CalculateWorldContactNormal();
								Data.PenetrationDepth = Constraint.GetPhi();
								Data.bProbe = Constraint.GetIsProbe();

								// @todo(chaos): fix this casting
								Data.Proxy1 = Particle0 ? const_cast<IPhysicsProxyBase*>(Particle0->PhysicsProxy()) : nullptr;
								Data.Proxy2 = Particle1 ? const_cast<IPhysicsProxyBase*>(Particle1->PhysicsProxy()) : nullptr;

								const FPerShapeData* Shape0 = Constraint.GetShape0();
								const FPerShapeData* Shape1 = Constraint.GetShape1();

								Data.ShapeIndex1 = Shape0 ? Shape0->GetShapeIndex() : INDEX_NONE;
								Data.ShapeIndex2 = Shape1 ? Shape1->GetShapeIndex() : INDEX_NONE;
								Data.Mat1 = ResolveMaterial(Shape0, Constraint);
								Data.Mat2 = ResolveMaterial(Shape1, Constraint);

								// Collision constraints require both proxies are valid. If either is not, we needn't record the collision event.
								if (Data.Proxy1 == nullptr || Data.Proxy2 == nullptr)
								{
									continue;
								}

								if (const FPBDRigidParticleHandle* Rigid0 = Particle0->CastToRigidParticle())
								{
									Data.DeltaVelocity1 = Rigid0->GetV() - Rigid0->GetPreV();
								}
								if (const FPBDRigidParticleHandle* Rigid1 = Particle1->CastToRigidParticle())
								{
									Data.DeltaVelocity2 = Rigid1->GetV() - Rigid1->GetPreV();
								}

								// todo: do we need these anymore now we are storing the particles you can access all of this stuff from there
								// do we still need these now we have pointers to particles returned?
								const FPBDRigidParticleHandle* PBDRigid0 = Particle0->CastToRigidParticle();
								if (PBDRigid0 && PBDRigid0->ObjectState() == EObjectStateType::Dynamic)
								{
									Data.Velocity1 = PBDRigid0->GetV();
									Data.AngularVelocity1 = PBDRigid0->GetW();
									Data.Mass1 = PBDRigid0->M();
								}

								const FPBDRigidParticleHandle* PBDRigid1 = Particle1->CastToRigidParticle();
								if (PBDRigid1 && PBDRigid1->ObjectState() == EObjectStateType::Dynamic)
								{
									Data.Velocity2 = PBDRigid1->GetV();
									Data.AngularVelocity2 = PBDRigid1->GetW();
									Data.Mass2 = PBDRigid1->M();
								}

								IPhysicsProxyBase* const PhysicsProxy = const_cast<IPhysicsProxyBase*>(Particle0->PhysicsProxy());
								IPhysicsProxyBase* const OtherPhysicsProxy = const_cast<IPhysicsProxyBase*>(Particle1->PhysicsProxy());

								const FSolverCollisionEventFilter* SolverCollisionEventFilter = Solver->GetEventFilters()->GetCollisionFilter();
								if (!SolverCollisionEventFilter->Enabled() || SolverCollisionEventFilter->Pass(Data))

								{
									DupAllCollisionsDataArray[IdxCollision] = Data;
#if TODO_REIMPLEMENT_RIGID_CLUSTERING
									// If Constraint.ParticleIndex is a cluster store an index for a mesh in this cluster
									if (ClusterIdsArray[Constraint.ParticleIndex].NumChildren > 0)
									{
										int32 ParticleIndexMesh = GetParticleIndexMesh(ParentToChildrenMap, Constraint.ParticleIndex);
										ensure(ParticleIndexMesh != INDEX_NONE);
										CollisionDataArrayItem.ParticleIndexMesh = ParticleIndexMesh;
									}
									// If Constraint.LevelsetIndex is a cluster store an index for a mesh in this cluster
									if (ClusterIdsArray[Constraint.LevelsetIndex].NumChildren > 0)
									{
										int32 LevelsetIndexMesh = GetParticleIndexMesh(ParentToChildrenMap, Constraint.LevelsetIndex);
										ensure(LevelsetIndexMesh != INDEX_NONE);
										CollisionDataArrayItem.LevelsetIndexMesh = LevelsetIndexMesh;
									}
#endif
								}
							}
						}, Chaos::GlobalSmallBatchSize);

						FCollisionDataArray& AllCollisionsDataArray = CollisionEventData.CollisionData.AllCollisionsArray;
						TMap<IPhysicsProxyBase*, TArray<int32>>& AllCollisionsIndicesByPhysicsProxy = CollisionEventData.PhysicsProxyToCollisionIndices.PhysicsProxyToIndicesMap;
						for (int32 IdxCollision = 0; IdxCollision < NumValidCollisions; ++IdxCollision)
						{
							if (DupAllCollisionsDataArray[IdxCollision].Proxy1 != nullptr && !DupAllCollisionsDataArray[IdxCollision].Proxy1->GetMarkedDeleted())
							{
								int32 NewIdx = AllCollisionsDataArray.Add(DupAllCollisionsDataArray[IdxCollision]);
								AllCollisionsIndicesByPhysicsProxy.FindOrAdd(AllCollisionsDataArray[NewIdx].Proxy1).Add(FEventManager::EncodeCollisionIndex(NewIdx, false));

								if (AllCollisionsDataArray[NewIdx].Proxy2 && AllCollisionsDataArray[NewIdx].Proxy2 != AllCollisionsDataArray[NewIdx].Proxy1 
									&& !AllCollisionsDataArray[NewIdx].Proxy2->GetMarkedDeleted())
								{
									AllCollisionsIndicesByPhysicsProxy.FindOrAdd(AllCollisionsDataArray[NewIdx].Proxy2).Add(FEventManager::EncodeCollisionIndex(NewIdx, true));
								}
							}
						}
					}
				}
			}
		});
	}

	void FEventDefaults::RegisterBreakingEvent(FEventManager& EventManager)
	{
		EventManager.template RegisterEvent<FBreakingEventData>(EEventType::Breaking, []
		(const Chaos::FPBDRigidsSolver* Solver, FBreakingEventData& BreakingEventData, bool bResetData)
		{
			check(Solver);

			EnsureIsInPhysicsThreadContext();

			SCOPE_CYCLE_COUNTER(STAT_GatherBreakingEvent);

			// #todo: This isn't working - SolverActor parameters are set on a solver but it is currently a different solver that is simulating!!
			if (!Solver->GetEventFilters()->IsBreakingEventEnabled())
			{
				return;
			}

			if (bResetData)
			{
				BreakingEventData.Reset();
			}
			BreakingEventData.BreakingData.TimeCreated = Solver->MTime;

			const auto* Evolution = Solver->GetEvolution();
			const FPBDRigidParticles& Particles = Evolution->GetParticles().GetDynamicParticles();
			const TArray<FBreakingData>& AllClusterBreakings = Evolution->GetRigidClustering().GetAllClusterBreakings();
			FBreakingDataArray& FilteredBreakingDataArray = BreakingEventData.BreakingData.AllBreakingsArray;
			TMap<IPhysicsProxyBase*, TArray<int32>>& FilteredBreakingIndicesByPhysicsProxy = BreakingEventData.PhysicsProxyToBreakingIndices.PhysicsProxyToIndicesMap;
	
			if (AllClusterBreakings.Num() > 0)
			{
				const FSolverBreakingEventFilter* SolverBreakingEventFilter = Solver->GetEventFilters()->GetBreakingFilter();
				for (int32 Idx = 0; Idx < AllClusterBreakings.Num(); ++Idx)
				{					
					const FBreakingData& ClusterBreaking = AllClusterBreakings[Idx];
					IPhysicsProxyBase* Proxy = AllClusterBreakings[Idx].Proxy;
					if (!Proxy->GetMarkedDeleted() && (!SolverBreakingEventFilter->Enabled() || SolverBreakingEventFilter->Pass(ClusterBreaking)))
					{
						const int32 NewIndex = FilteredBreakingDataArray.Emplace(ClusterBreaking);
						FilteredBreakingIndicesByPhysicsProxy.FindOrAdd(Proxy).Add(FEventManager::EncodeCollisionIndex(NewIndex, false));
						BreakingEventData.BreakingData.bHasGlobalEvent |= (ClusterBreaking.EmitterFlag & EventEmitterFlag::GlobalDispatcher) != 0;
					}
				}
			}
		});
	}

	void FEventDefaults::RegisterTrailingEvent(FEventManager& EventManager)
	{
		EventManager.template RegisterEvent<FTrailingEventData>(EEventType::Trailing, []
		(const Chaos::FPBDRigidsSolver* Solver, FTrailingEventData& TrailingEventData, bool bResetData)
		{
			check(Solver);

			EnsureIsInPhysicsThreadContext();

			// #todo: This isn't working - SolverActor parameters are set on a solver but it is currently a different solver that is simulating!!
			if (!Solver->GetEventFilters()->IsTrailingEventEnabled())
				return;

			const auto* Evolution = Solver->GetEvolution();

			const TArrayCollectionArray<ClusterId>& ClusterIdsArray = Evolution->GetRigidClustering().GetClusterIdsArray();
#if TODO_REIMPLEMENT_RIGID_CLUSTERING
			const TMap<uint32, TUniquePtr<TArray<uint32>>>& ParentToChildrenMap = Evolution->GetRigidClustering().GetChildrenMap();
#endif


			if (bResetData)
			{
				TrailingEventData.Reset();
			}
			TrailingEventData.TrailingData.TimeCreated = Solver->MTime;
			TrailingEventData.PhysicsProxyToTrailingIndices.TimeCreated = Solver->MTime;

			const TArray<TPBDRigidParticleHandle<Chaos::FReal, 3>*>& ActiveParticlesArray = Evolution->GetParticles().GetActiveParticlesArray();
			FTrailingDataArray& AllTrailingsDataArray = TrailingEventData.TrailingData.AllTrailingsArray;
			TMap<IPhysicsProxyBase*, TArray<int32>>& AllTrailingIndicesByPhysicsProxy = TrailingEventData.PhysicsProxyToTrailingIndices.PhysicsProxyToIndicesMap;

			for (TPBDRigidParticleHandle<Chaos::FReal, 3>* ActiveParticle : ActiveParticlesArray)
			{

				if (ensure(FMath::IsFinite(ActiveParticle->InvM())))
				{
					if (ActiveParticle->InvM() != 0.f &&
						ActiveParticle->GetGeometry() &&
						ActiveParticle->GetGeometry()->HasBoundingBox())
					{
						if (ensure(!ActiveParticle->GetX().ContainsNaN() &&
							!ActiveParticle->GetV().ContainsNaN() &&
							!ActiveParticle->GetW().ContainsNaN() &&
							FMath::IsFinite(ActiveParticle->M())))
						{
							FTrailingData TrailingData;
							TrailingData.Location = ActiveParticle->GetX();
							TrailingData.Orientation = ActiveParticle->GetR();
							TrailingData.Velocity = ActiveParticle->GetV();
							TrailingData.AngularVelocity = ActiveParticle->GetW();
							TrailingData.Mass = ActiveParticle->M();
							TrailingData.Proxy = ActiveParticle->PhysicsProxy();
							
							if (ActiveParticle->GetGeometry()->HasBoundingBox())
							{
								TrailingData.BoundingBox = ActiveParticle->GetGeometry()->BoundingBox();
							}

							if (TrailingData.Proxy->GetType() == EPhysicsProxyType::GeometryCollectionType)
							{
								FGeometryCollectionPhysicsProxy* ConcreteProxy = static_cast<FGeometryCollectionPhysicsProxy*>(TrailingData.Proxy);
								TrailingData.TransformGroupIndex = ConcreteProxy->GetTransformGroupIndexFromHandle(ActiveParticle);
								TrailingData.SetEmitterFlag(ConcreteProxy->GetSimParameters().bGenerateTrailingData, ConcreteProxy->GetSimParameters().bGenerateGlobalTrailingData);
							}
							else
							{
								TrailingData.TransformGroupIndex = INDEX_NONE;
							}

							const FSolverTrailingEventFilter* SolverTrailingEventFilter = Solver->GetEventFilters()->GetTrailingFilter();
							if (!SolverTrailingEventFilter->Enabled() || SolverTrailingEventFilter->Pass(TrailingData))
							{
								const int32 NewIdx = AllTrailingsDataArray.Emplace(TrailingData);

								// Add to AllTrailingIndicesByPhysicsProxy
								AllTrailingIndicesByPhysicsProxy.FindOrAdd(TrailingData.Proxy).Add(FEventManager::EncodeCollisionIndex(NewIdx, false));
								TrailingEventData.TrailingData.bHasGlobalEvent |= (TrailingData.EmitterFlag & EventEmitterFlag::GlobalDispatcher) != 0;
							}
						}
					}
				}
			}
		});
	}

	void FEventDefaults::RegisterSleepingEvent(FEventManager& EventManager)
	{
		EventManager.template RegisterEvent<FSleepingEventData>(EEventType::Sleeping, []
		(const Chaos::FPBDRigidsSolver* Solver, FSleepingEventData& SleepingEventData, bool bResetData)
		{
			check(Solver);

			EnsureIsInPhysicsThreadContext();

			SCOPE_CYCLE_COUNTER(STAT_GatherSleepingEvent);

			const auto* Evolution = Solver->GetEvolution();

			SleepingEventData.Reset();

			Chaos::FPBDRigidsSolver* NonConstSolver = const_cast<Chaos::FPBDRigidsSolver*>(Solver);

			const TArray<FPBDRigidParticles*> RelevantParticleArrays = {
				&NonConstSolver->Particles.GetDynamicParticles(),
				&NonConstSolver->Particles.GetClusteredParticles(),
				&NonConstSolver->Particles.GetGeometryCollectionParticles()
			};

			FSleepingDataArray& EventSleepDataArray = SleepingEventData.SleepingData;
			for (FPBDRigidParticles* ParticleArray : RelevantParticleArrays)
			{
				check(ParticleArray != nullptr);
				ParticleArray->GetSleepDataLock().ReadLock();
				const TArray<TSleepData<FReal, 3>>& SolverSleepingData = ParticleArray->GetSleepData();
				for (const TSleepData<FReal, 3>& SleepData : SolverSleepingData)
				{
					if (SleepData.Particle)
					{
						FGeometryParticle* Particle = SleepData.Particle->GTGeometryParticle();
						if (Particle && SleepData.Particle->PhysicsProxy())
						{
							int32 NewIdx = EventSleepDataArray.Add(FSleepingData());
							FSleepingData& SleepingDataArrayItem = EventSleepDataArray[NewIdx];
							SleepingDataArrayItem.Proxy = SleepData.Particle->PhysicsProxy();
							SleepingDataArrayItem.Sleeping = SleepData.Sleeping;
						}
					}
				}
				ParticleArray->GetSleepDataLock().ReadUnlock();
				ParticleArray->ClearSleepData();
			}

			// We don't care about sleep data added to these
			NonConstSolver->Particles.GetDynamicKinematicParticles().ClearSleepData();
			NonConstSolver->Particles.GetDynamicDisabledParticles().ClearSleepData();
		});
	}

	void FEventDefaults::RegisterRemovalEvent(FEventManager& EventManager)
	{
		EventManager.template RegisterEvent<FRemovalEventData>(EEventType::Removal, []
		(const Chaos::FPBDRigidsSolver* Solver, FRemovalEventData& RemovalEventData, bool bResetData)
			{
				check(Solver);
				EnsureIsInPhysicsThreadContext();

				if (bResetData)
				{
					RemovalEventData.Reset();
				}
				RemovalEventData.RemovalData.TimeCreated = Solver->MTime;

				const TArray<FRemovalData>& AllRemovalsArray = Solver->GetEvolution()->GetAllRemovals();
				FRemovalDataArray& AllRemovalDataArray = RemovalEventData.RemovalData.AllRemovalArray;
				TMap<IPhysicsProxyBase*, TArray<int32>>& AllRemovalIndicesByPhysicsProxy = RemovalEventData.PhysicsProxyToRemovalIndices.PhysicsProxyToIndicesMap;
				
				for (int32 Idx = 0; Idx < AllRemovalsArray.Num(); ++Idx)
				{
					IPhysicsProxyBase* Proxy = AllRemovalsArray[Idx].Proxy;
					if (!Proxy->GetMarkedDeleted())
					{
						FRemovalData RemovalData;
						RemovalData.Location = AllRemovalsArray[Idx].Location;
						RemovalData.Mass = AllRemovalsArray[Idx].Mass;
						RemovalData.Proxy = Proxy;
						RemovalData.BoundingBox = AllRemovalsArray[Idx].BoundingBox;

						const int32 NewIdx = AllRemovalDataArray.Add(RemovalData);
						AllRemovalIndicesByPhysicsProxy.FindOrAdd(Proxy).Add(FEventManager::EncodeCollisionIndex(NewIdx, false));
					}
				}
			});
	}

	void FEventDefaults::RegisterCrumblingEvent(FEventManager& EventManager)
	{
		EventManager.template RegisterEvent<FCrumblingEventData>(EEventType::Crumbling, []
		(const Chaos::FPBDRigidsSolver* Solver, FCrumblingEventData& CrumblingEventData, bool bResetData)
		{
			check(Solver);

			EnsureIsInPhysicsThreadContext();

			SCOPE_CYCLE_COUNTER(STAT_GatherBreakingEvent);

			// @todo(chaos) : should we check for a way to globally stop this from being processed ? 

			if (bResetData)
			{
				CrumblingEventData.Reset();
			}
			CrumblingEventData.SetTimeCreated(Solver->MTime);

			const TArray<FCrumblingData>& AllCrumblingsArray = Solver->GetEvolution()->GetRigidClustering().GetAllClusterCrumblings();

			CrumblingEventData.Reserve(AllCrumblingsArray.Num());
			for (const FCrumblingData& Crumbling: AllCrumblingsArray)
			{
				// todo(chaos) : implement filtering only if necessary as crumbling event should be sparser than breaking ones
				CrumblingEventData.AddCrumbling(Crumbling);
				CrumblingEventData.CrumblingData.bHasGlobalEvent |= (Crumbling.EmitterFlag & EventEmitterFlag::GlobalDispatcher) != 0;
			}
		});
	}
}
