// Copyright Epic Games, Inc. All Rights Reserved.

#include "EventManager.h"

namespace Chaos
{
	namespace Private
	{
		// This is required to be able to track continuous collisions (so that we can detect the "no collision" state)
		bool bEventManagerDispatchEmptyEvents = true;
		FAutoConsoleVariableRef CVarEventManagerDispatchEmptyEvent(TEXT("p.Chaos.EventManager.DispatchEmptyEvents"), bEventManagerDispatchEmptyEvents, TEXT("Whether to dispatch events even if the event buffers are empty"));

		bool bUseTransactionallySafeSharedRecursiveMutex = true;
		FAutoConsoleVariableRef CVarUseUseTransactionallySafeSharedRecursiveMutex(TEXT("Chaos.UseTransactionallySafeSharedMutex"), bUseTransactionallySafeSharedRecursiveMutex, TEXT("Sets the Event Manager to use TransactionallySafeSharedRecursiveMutexes to avoid a pontential recursive hang."));
	}


	FEventManager::FEventManager(const Chaos::EMultiBufferMode& BufferModeIn)
		: BufferMode(BufferModeIn)
	{
		if (Private::bUseTransactionallySafeSharedRecursiveMutex)
		{
			ContainerLock.Emplace<UE::FTransactionallySafeSharedRecursiveMutex>();
			ResourceLock.Emplace<UE::FTransactionallySafeSharedRecursiveMutex>();
		}
		else
		{
			ContainerLock.Emplace<FTransactionallySafeRWLock>();
			ResourceLock.Emplace<FTransactionallySafeRWLock>();
		}
	}

	void FEventManager::Reset()
	{
		FLockWriteScope ContainerWriteScope(ContainerLock);
		for (FEventContainerBasePtr Container : EventContainers)
		{
			delete Container;
			Container = nullptr;
		}
		EventContainers.Reset();
	}

	void FEventManager::UnregisterEvent(const EEventType& EventType)
	{
		const FEventID EventID = (FEventID)EventType;
		FLockWriteScope ContainerWriteScope(ContainerLock);
		if (EventID < EventContainers.Num())
		{
			delete EventContainers[EventID];
			EventContainers.RemoveAt(EventID);
		}
	}

	void FEventManager::UnregisterHandler(const EEventType& EventType, const void* InHandler)
	{
		const FEventID EventID = (FEventID)EventType;
		FLockReadScope ContainerReadScope(ContainerLock);
		checkf(EventID < EventContainers.Num(), TEXT("Unregistering event Handler for an event ID that does not exist"));
		EventContainers[EventID]->UnregisterHandler(InHandler);
	}

	void FEventManager::FillProducerData(const Chaos::FPBDRigidsSolver* Solver, bool bResetData)
	{
		auto ContainerLambda = [this, Solver, bResetData]()
			{
				FLockReadScope ContainerReadScope(ContainerLock);
				for (FEventContainerBasePtr EventContainer : EventContainers)
				{
					if (EventContainer)
					{
						EventContainer->InjectProducerData(Solver, bResetData);
					}
				}
			};

		if (BufferMode == EMultiBufferMode::Double)
		{
			FLockReadScope ResourceReadScope(ResourceLock);
			ContainerLambda();
		}

		else
		{
			ContainerLambda();
		}
	}

	void FEventManager::FlipBuffersIfRequired()
	{
		auto ContainerLambda = [this]()
			{
				FLockReadScope ContainerReadScope(ContainerLock);
				for (FEventContainerBasePtr EventContainer : EventContainers)
				{
					if (EventContainer)
					{
						EventContainer->ResetConsumerBuffer();
						EventContainer->FlipBufferIfRequired();
					}
				}
			};

		if (BufferMode == EMultiBufferMode::Single)
		{
			return;
		}
		else if (BufferMode == EMultiBufferMode::Double)
		{
			FLockWriteScope ResourceWriteScope(ResourceLock);
			ContainerLambda();
		}
		else
		{
			ContainerLambda();
		}
	}

	void FEventManager::DispatchEvents()
	{
		auto ContainerLambda = [this]()
			{
				FLockReadScope ContainerReadScope(ContainerLock);
				for (FEventContainerBasePtr EventContainer : EventContainers)
				{
					if (EventContainer)
					{
						EventContainer->DispatchConsumerData();
					}
				}
			};

		if (BufferMode == EMultiBufferMode::Double)
		{
			FLockReadScope ResourceReadScope(ResourceLock);
			ContainerLambda();
		}
		else
		{
			ContainerLambda();
		}
		

		if (BufferMode == EMultiBufferMode::Single)
		{
			for (FEventContainerBasePtr EventContainer : EventContainers)
			{
				if (EventContainer)
				{
					EventContainer->ResetConsumerBuffer();
					EventContainer->FlipBufferIfRequired();
				}
			}
		}
	}

	void FEventManager::InternalRegisterInjector(const FEventID& EventID, const FEventContainerBasePtr& Container)
	{
		if (EventID > EventContainers.Num())
		{
			for (int i = EventContainers.Num(); i < EventID; i++)
			{
				EventContainers.Push(nullptr);
			}
		}

		EventContainers.EmplaceAt(EventID, Container);
	}

	int32 FEventManager::EncodeCollisionIndex(int32 ActualCollisionIndex, bool bSwapOrder)
	{
		return bSwapOrder ? (ActualCollisionIndex | (1 << 31)) : ActualCollisionIndex;
	}

	int32 FEventManager::DecodeCollisionIndex(int32 EncodedCollisionIdx, bool& bSwapOrder)
	{
		bSwapOrder = EncodedCollisionIdx & (1 << 31);
		return EncodedCollisionIdx & ~(1 << 31);
	}
}
