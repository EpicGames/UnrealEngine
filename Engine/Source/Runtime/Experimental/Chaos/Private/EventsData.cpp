// Copyright Epic Games, Inc. All Rights Reserved.
#include "EventsData.h"
#include "EventManager.h"

namespace Chaos
{
	FCollisionEventDataIterator::FCollisionEventDataIterator(const TArrayView<const IPhysicsProxyBase*>& InProxies, const Chaos::FCollisionEventData& InCollisionEventData)
		: ProxyIndex(INDEX_NONE)
		, ProxyCollisionIndex(INDEX_NONE)
		, ProxyCollisionIndices(nullptr)
		, Proxies(InProxies)
		, CollisionEventData(InCollisionEventData)
	{
		Reset();
	}

	void FCollisionEventDataIterator::Reset()
	{
		ProxyIndex = 0;
		ProxyCollisionIndex = 0;
		ProxyCollisionIndices = nullptr;

		if (ProxyIndex < Proxies.Num())
		{
			const IPhysicsProxyBase* NextProxy = Proxies[ProxyIndex];
			ProxyCollisionIndices = CollisionEventData.PhysicsProxyToCollisionIndices.PhysicsProxyToIndicesMap.Find(NextProxy);
		}

		while (!IsFinished() && !GetCurrentCollidingDataIndex().IsValid())
		{
			Next();
		}
	}

	void FCollisionEventDataIterator::Next()
	{
		if (!IsFinished())
		{
			// Move to the next event for the current proxy
			++ProxyCollisionIndex;

			// If we run out of events for that proxy, move to the next proxy
			// Skip over any proxies with no collision events
			// NOTE: Next() may be from Reset() so we have to handle null ProxyCollisionIndices
			if ((ProxyCollisionIndices == nullptr) || (ProxyCollisionIndex >= ProxyCollisionIndices->Num()))
			{
				ProxyCollisionIndex = 0;
				ProxyCollisionIndices = nullptr;
				while (!IsFinished() && !GetCurrentCollidingDataIndex().IsValid())
				{
					if (++ProxyIndex < Proxies.Num())
					{
						const IPhysicsProxyBase* CurrentProxy = Proxies[ProxyIndex];
						ProxyCollisionIndices = CollisionEventData.PhysicsProxyToCollisionIndices.PhysicsProxyToIndicesMap.Find(CurrentProxy);

						// Skip empty proxies
						if ((ProxyCollisionIndices != nullptr) && (ProxyCollisionIndices->Num() == 0))
						{
							ProxyCollisionIndices = nullptr;
						}
					}
				}
			}
		}
	}

	FCollidingDataIndex FCollisionEventDataIterator::GetCurrentCollidingDataIndex() const
	{
		if (!IsFinished())
		{
			if ((ProxyCollisionIndices != nullptr) && (ProxyCollisionIndex < ProxyCollisionIndices->Num()))
			{
				bool bSwapProxyOrder;
				int32 EncodedCollisionIndex = (*ProxyCollisionIndices)[ProxyCollisionIndex];
				int32 CollisionIndex = FEventManager::DecodeCollisionIndex(EncodedCollisionIndex, bSwapProxyOrder);
				int32 CollisionProxyIndex = bSwapProxyOrder ? 1 : 0;
				return FCollidingDataIndex(CollisionIndex, CollisionProxyIndex);
			}
		}
		return {};
	}

	const IPhysicsProxyBase* FCollisionEventDataIterator::GetCurrentProxy() const
	{
		if (!IsFinished())
		{
			return Proxies[ProxyIndex];
		}
		return nullptr;
	}
}