// Copyright Epic Games, Inc. All Rights Reserved.

#include "Field/FieldSystem.h"
#include "Field/FieldSystemNodes.h"

void FFieldSystemCommand::Serialize(FArchive& Ar)
{
	Ar << TargetAttribute;

	uint8 dType = RootNode.IsValid() ? (uint8)RootNode->Type() : (uint8)FFieldNodeBase::EFieldType::EField_None;
	Ar << dType;

	uint8 sType = RootNode.IsValid()?(uint8)RootNode->SerializationType(): (uint8)FFieldNodeBase::ESerializationType::FieldNode_Null;
	Ar << sType;

	if (Ar.IsLoading())
	{
		RootNode.Reset(FieldNodeFactory((FFieldNodeBase::EFieldType)dType,(FFieldNodeBase::ESerializationType)sType));
	}

	if (RootNode.IsValid())
	{
		RootNode->Serialize(Ar);
	}

	//Ar << CommandName;
	//Ar << TimeCreation;

	// @todo: Add MetaData serialization support. 
}

bool FFieldSystemCommand::operator==(const FFieldSystemCommand& CommandIn) const
{
	if (TargetAttribute.IsEqual(CommandIn.TargetAttribute) && CommandName.IsEqual(CommandIn.CommandName) && (BoundingBox == CommandIn.BoundingBox) && (MaxMagnitude == CommandIn.MaxMagnitude))
	{
		if (RootNode.IsValid() == CommandIn.RootNode.IsValid())
		{
			if (RootNode.IsValid())
			{
				if (RootNode->SerializationType() == CommandIn.RootNode->SerializationType())
				{
					if (RootNode->operator==(*CommandIn.RootNode))
					{
						return true;
					}
				}
			}
			else
			{
				return true;
			}
		}
	}
	return false;
}
