// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GeometryCollection.cpp: UGeometryCollection methods.
=============================================================================*/

#include "Field/FieldSystemCoreAlgo.h"
#include "Field/FieldSystemNodes.h"
#include "Async/ParallelFor.h"

DEFINE_LOG_CATEGORY_STATIC(FSCA_Log, Log, All);

namespace FieldSystemAlgo
{
}
