// Copyright Epic Games, Inc. All Rights Reserved.

#include "Field/FieldSystemNodes.h"
#include "Field/FieldSystemNoiseAlgo.h"
#include "Async/ParallelFor.h"
#include "Chaos/Vector.h"
#include <type_traits>

namespace 
{
	inline FVector MinVector(const FVector& VectorA, const FVector& VectorB)
	{
		return FVector(FMath::Min(VectorA.X, VectorB.X), FMath::Min(VectorA.Y, VectorB.Y), FMath::Min(VectorA.Z, VectorB.Z));
	}

	inline FVector MaxVector(const FVector& VectorA, const FVector& VectorB)
	{
		return FVector(FMath::Max(VectorA.X, VectorB.X), FMath::Max(VectorA.Y, VectorB.Y), FMath::Max(VectorA.Z, VectorB.Z));
	}
}

FFieldNodeBase * FieldNodeFactory(FFieldNodeBase::EFieldType BaseType,FFieldNodeBase::ESerializationType Type)
{
	switch (Type)
	{
	case FFieldNodeBase::ESerializationType::FieldNode_FUniformInteger:
		return new FUniformInteger();
	case FFieldNodeBase::ESerializationType::FieldNode_FRadialIntMask:
		return new FRadialIntMask();
	case FFieldNodeBase::ESerializationType::FieldNode_FUniformScalar:
		return new FUniformScalar();
	case FFieldNodeBase::ESerializationType::FieldNode_FWaveScalar:
		return new FWaveScalar();
	case FFieldNodeBase::ESerializationType::FieldNode_FRadialFalloff:
		return new FRadialFalloff();
	case FFieldNodeBase::ESerializationType::FieldNode_FPlaneFalloff:
		return new FPlaneFalloff();
	case FFieldNodeBase::ESerializationType::FieldNode_FBoxFalloff:
		return new FBoxFalloff();
	case FFieldNodeBase::ESerializationType::FieldNode_FNoiseField:
		return new FNoiseField();
	case FFieldNodeBase::ESerializationType::FieldNode_FUniformVector:
		return new FUniformVector();
	case FFieldNodeBase::ESerializationType::FieldNode_FRadialVector:
		return new FRadialVector();
	case FFieldNodeBase::ESerializationType::FieldNode_FRotatedRadialVector:
		return new FRotatedRadialVector();
	case FFieldNodeBase::ESerializationType::FieldNode_FRandomVector:
		return new FRandomVector();
	case FFieldNodeBase::ESerializationType::FieldNode_FSumScalar:
		return new FSumScalar();
	case FFieldNodeBase::ESerializationType::FieldNode_FSumVector:
		return new FSumVector();
	case FFieldNodeBase::ESerializationType::FieldNode_FConversionField:
		if(BaseType==FFieldNodeBase::EFieldType::EField_Int32)
			return new FConversionField<float,int32>();
		else if (BaseType == FFieldNodeBase::EFieldType::EField_Float)
			return new FConversionField<int32,float>();
		break;
	case FFieldNodeBase::ESerializationType::FieldNode_FCullingField:
		if (BaseType == FFieldNodeBase::EFieldType::EField_Int32)
			return new FCullingField<int32>();
		if (BaseType == FFieldNodeBase::EFieldType::EField_Float)
			return new FCullingField<float>();
		if (BaseType == FFieldNodeBase::EFieldType::EField_FVector)
			return new FCullingField<FVector>();
		break;
	case FFieldNodeBase::ESerializationType::FieldNode_FReturnResultsTerminal:
		if(BaseType==FFieldNodeBase::EFieldType::EField_Int32)
			return new FReturnResultsTerminal<int32>();
		else if (BaseType == FFieldNodeBase::EFieldType::EField_Float)
			return new FReturnResultsTerminal<float>();
		else if (BaseType == FFieldNodeBase::EFieldType::EField_FVector)
			return new FReturnResultsTerminal<FVector>();
		break;
	}
	return nullptr;
}


template<class T>
void SerializeInternal(FArchive& Ar, TUniquePtr< FFieldNode<T> >& Field)
{
	uint8 dType = Field.IsValid() ? (uint8)Field->Type() : (uint8)FFieldNodeBase::EFieldType::EField_None;
	Ar << dType;

	uint8 sType = Field.IsValid() ? (uint8)Field->SerializationType() : (uint8)FFieldNodeBase::ESerializationType::FieldNode_Null;
	Ar << sType;

	if (Ar.IsLoading())
	{
		Field.Reset(static_cast<FFieldNode<T>* >(FieldNodeFactory((FFieldNodeBase::EFieldType)dType, (FFieldNodeBase::ESerializationType)sType)));
	}

	if (Field.IsValid())
	{
		Field->Serialize(Ar);
	}
}
template<class Enum>
void SerializeInternal(FArchive& Ar, Enum& Var)
{
	uint8 Type = (uint8)Var;
	Ar << Type;
	Var = (Enum)Type;
}


/**
* FUniformInteger
*/

void FUniformInteger::Evaluate(FFieldContext& Context, TFieldArrayView<int32>& Results) const
{
	int32 MagnitudeVal = Magnitude;

	int32 NumSamples = Context.SampleIndices.Num();
	for (int32 SampleIndex = 0; SampleIndex < NumSamples; SampleIndex++)
	{
		Results[Context.SampleIndices[SampleIndex].Result] = MagnitudeVal;
	}
}

void FUniformInteger::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar << Magnitude;
}

bool FUniformInteger::operator==(const FFieldNodeBase& Node)
{
	if (Node.SerializationType() == SerializationType())
	{
		return Super::operator==(Node)
			&& Magnitude == static_cast<const FUniformInteger*>(&Node)->Magnitude;
	}
	return false;
}

void FUniformInteger::FillSetupCount(int32& NumOffsets, int32& NumParams) const
{
	Super::FillSetupCount(NumOffsets, NumParams);
	NumParams += 1;
}

void FUniformInteger::FillSetupDatas(TArray<int32>& NodesOffsets, TArray<float>& NodesParams, const float CommandTime) const
{
	Super::FillSetupDatas(NodesOffsets, NodesParams, CommandTime);
	NodesParams.Add(static_cast<float>(Magnitude));
}

float FUniformInteger::EvalMaxMagnitude() const
{
	return static_cast<float>(Magnitude);
}

/**
* FRadialIntMask
*/

void FRadialIntMask::Evaluate(FFieldContext& Context, TFieldArrayView<int32>& Results) const
{
	float Radius2 = Radius * Radius;

	int32 NumSamples = Context.SampleIndices.Num();
	for (int32 SampleIndex = 0; SampleIndex < NumSamples; SampleIndex++)
	{
		const FFieldContextIndex& Index = Context.SampleIndices[SampleIndex];
		{
			int32 Result;
			FVector::FReal Delta2 = (Position - Context.SamplePositions[Index.Sample]).SizeSquared();

			if(Delta2 < Radius2)
			{
				Result = InteriorValue;
			}
			else
			{
				Result = ExteriorValue;
			}

			switch (SetMaskCondition) 
			{
			case ESetMaskConditionType::Field_Set_Always:
				Results[Index.Result] = Result;
				break;

			case ESetMaskConditionType::Field_Set_IFF_NOT_Interior:
				if (Results[Index.Result] != InteriorValue) 
				{
					Results[Index.Result] = Result;
				}
				break;

			case ESetMaskConditionType::Field_Set_IFF_NOT_Exterior:
				if (Results[Index.Result] != ExteriorValue) 
				{
					Results[Index.Result] = Result;
				}
				break;
			}
		}
	}
}

void FRadialIntMask::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar << Radius;
	Ar << Position;
	Ar << InteriorValue;
	Ar << ExteriorValue;
	SerializeInternal<ESetMaskConditionType>(Ar, SetMaskCondition);
}

bool FRadialIntMask::operator==(const FFieldNodeBase& Node)
{
	if (Node.SerializationType() == SerializationType())
	{
		const FRadialIntMask* Other = static_cast<const FRadialIntMask*>(&Node);
		return Super::operator==(Node)
			&& Radius == Other->Radius
			&& Position==Other->Position
			&& InteriorValue==Other->InteriorValue
			&& ExteriorValue==Other->ExteriorValue
			&& SetMaskCondition== Other->SetMaskCondition;
	}
	return false;
}

void FRadialIntMask::FillSetupCount(int32& NumOffsets, int32& NumParams) const
{
	Super::FillSetupCount(NumOffsets, NumParams);
	NumParams += 7;
}

void FRadialIntMask::FillSetupDatas(TArray<int32>& NodesOffsets, TArray<float>& NodesParams, const float CommandTime) const
{
	Super::FillSetupDatas(NodesOffsets, NodesParams, CommandTime);
	NodesParams.Add(Radius);
	NodesParams.Add(static_cast<float>(Position.X));
	NodesParams.Add(static_cast<float>(Position.Y));
	NodesParams.Add(static_cast<float>(Position.Z));
	NodesParams.Add(static_cast<float>(InteriorValue));
	NodesParams.Add(static_cast<float>(ExteriorValue));
	NodesParams.Add(static_cast<float>(SetMaskCondition));
}

float FRadialIntMask::EvalMaxMagnitude() const
{
	return 1.0f;
}

void FRadialIntMask::ComputeFieldBounds(FVector& MinBounds, FVector& MaxBounds, FVector& CenterPosition) const
{
	MinBounds = (ExteriorValue == 0) ? Position - FVector(Radius) : FVector(-FLT_MAX);
	MaxBounds = (ExteriorValue == 0) ? Position + FVector(Radius) : FVector(FLT_MAX);
	CenterPosition = Position;
}

/**
* FUniformScalar
*/

void FUniformScalar::Evaluate(FFieldContext& Context, TFieldArrayView<float>& Results) const
{
	float MagnitudeVal = Magnitude;

	int32 NumSamples = Context.SampleIndices.Num();
	for (int32 SampleIndex = 0; SampleIndex < NumSamples; SampleIndex++)
	{
		Results[Context.SampleIndices[SampleIndex].Result] = MagnitudeVal;
	}
}

void FUniformScalar::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar << Magnitude;
}

bool FUniformScalar::operator==(const FFieldNodeBase& Node)
{
	if (Node.SerializationType() == SerializationType())
	{
		return Super::operator==(Node)
			&& Magnitude == static_cast<const FUniformScalar*>(&Node)->Magnitude;
	}
	return false;
}

void FUniformScalar::FillSetupCount(int32& NumOffsets, int32& NumParams) const
{
	Super::FillSetupCount(NumOffsets, NumParams);
	NumParams += 1;
}

void FUniformScalar::FillSetupDatas(TArray<int32>& NodesOffsets, TArray<float>& NodesParams, const float CommandTime) const
{
	Super::FillSetupDatas(NodesOffsets, NodesParams, CommandTime);
	NodesParams.Add(Magnitude);
}

float FUniformScalar::EvalMaxMagnitude() const
{
	return Magnitude;
}

/**
* FWaveScalar
*/
void FWaveScalar::Evaluate(FFieldContext& Context, TFieldArrayView<float>& Results) const
{
	const float Velocity = (Period != 0.0f ) ? Wavelength / Period : 0.0f;

	const float Wavenumber = (Wavelength != 0.0f ) ? 2.0f * UE_PI / Wavelength : 0.0f;
	const Chaos::FReal DeltaTime = FMath::Max(Context.TimeSeconds, 0.0f);
	const Chaos::FReal Radius = Wavelength * DeltaTime / Period;
	const Chaos::FReal Decay = DeltaTime / Period;

	int32 NumSamples = Context.SampleIndices.Num();
	for (int32 SampleIndex = 0; SampleIndex < NumSamples; SampleIndex++)
	{
		const FFieldContextIndex& Index = Context.SampleIndices[SampleIndex];

		if (Function == EWaveFunctionType::Field_Wave_Decay)
		{
			Results[Index.Result] = Magnitude * (float)FMath::Exp(-Decay * Decay);							// LWC_TODO: Precision loss
		}
		else
		{
			const Chaos::FReal Distance = (float)(Context.SamplePositions[Index.Sample] - Position).Size();
			const Chaos::FReal Fraction = (1.0f - Distance / Radius);
			const Chaos::FReal Phase = -Wavenumber * Radius * Fraction;

			if (Function == EWaveFunctionType::Field_Wave_Cosine)
			{
				Results[Index.Result] = Magnitude * (float)FMath::Cos(Phase);								// LWC_TODO: Precision loss
			}
			else if (Function == EWaveFunctionType::Field_Wave_Gaussian)
			{
				Results[Index.Result] = Magnitude * (float)FMath::Exp(-Phase * Phase);						// LWC_TODO: Precision loss
			}
			else if (Function == EWaveFunctionType::Field_Wave_Falloff)
			{
				if (Distance < Radius && Radius > 0)
				{
					if (Falloff == EFieldFalloffType::Field_FallOff_None)
					{
						Results[Index.Result] = Magnitude;
					}
					else if (Falloff == EFieldFalloffType::Field_Falloff_Linear)
					{
						Results[Index.Result] = Magnitude * float(Fraction);								// LWC_TODO: Precision loss
					}
					else if (Falloff == EFieldFalloffType::Field_Falloff_Squared)
					{
						Results[Index.Result] = Magnitude * float(Fraction * Fraction);						// LWC_TODO: Precision loss
					}
					else if (Falloff == EFieldFalloffType::Field_Falloff_Inverse && Fraction > 0.0f)
					{
						Results[Index.Result] = Magnitude * 2.0f * (1.0f - 1.0f / float(Fraction + 1.0f));	// LWC_TODO: Precision loss
					}
					else if (Falloff == EFieldFalloffType::Field_Falloff_Logarithmic)
					{
						Results[Index.Result] = Magnitude * FMath::LogX(2.0f, (float)Fraction + 1.0f);		// LWC_TODO: Precision loss
					}
				}
				else
				{
					Results[Index.Result] = 0.0f;
				}
			}
		}
	}
}

void FWaveScalar::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar << Magnitude;
	Ar << Position;
	Ar << Wavelength;
	Ar << Period;
	SerializeInternal<EWaveFunctionType>(Ar, Function);
	SerializeInternal<EFieldFalloffType>(Ar, Falloff);
}

bool FWaveScalar::operator==(const FFieldNodeBase& Node)
{
	if (Node.SerializationType() == SerializationType())
	{
		return Super::operator==(Node)
			&& Magnitude == static_cast<const FWaveScalar*>(&Node)->Magnitude
			&& Position == static_cast<const FWaveScalar*>(&Node)->Position
			&& Wavelength == static_cast<const FWaveScalar*>(&Node)->Wavelength
			&& Period == static_cast<const FWaveScalar*>(&Node)->Period
			&& Function == static_cast<const FWaveScalar*>(&Node)->Function
			&& Falloff == static_cast<const FWaveScalar*>(&Node)->Falloff;
	}
	return false;
}

void FWaveScalar::FillSetupCount(int32& NumOffsets, int32& NumParams) const
{
	Super::FillSetupCount(NumOffsets, NumParams);
	NumParams += 9;
}

void FWaveScalar::FillSetupDatas(TArray<int32>& NodesOffsets, TArray<float>& NodesParams,const float CommandTime) const
{
	Super::FillSetupDatas(NodesOffsets, NodesParams, CommandTime);
	NodesParams.Add(Magnitude);
	NodesParams.Add(static_cast<float>(Position.X));
	NodesParams.Add(static_cast<float>(Position.Y));
	NodesParams.Add(static_cast<float>(Position.Z));
	NodesParams.Add(Wavelength);
	NodesParams.Add(Period);
	NodesParams.Add(CommandTime);
	NodesParams.Add(static_cast<float>(Function));
	NodesParams.Add(static_cast<float>(Falloff));
}

float FWaveScalar::EvalMaxMagnitude() const
{
	return Magnitude;
}

void FWaveScalar::ComputeFieldBounds(FVector& MinBounds, FVector& MaxBounds, FVector& CenterPosition) const
{
	MinBounds = FVector(-FLT_MAX);
	MaxBounds = FVector(FLT_MAX);
	CenterPosition = Position;
}

/**
* Function Utils
*/

float ScaleFunctionResult(const float& MinRange, const float& DeltaRange, const float& NodeMagnitude, const float& FunctionResult)
{
	return NodeMagnitude * (MinRange + DeltaRange * FunctionResult);
}

template<EFieldFalloffType FalloffType>
float EvalFalloffFunction(const float& MinRange, const float& DeltaRange, const float& NodeMagnitude, const float& FalloffValue)
{ 
	return 0.0;
}

template<>
float EvalFalloffFunction<EFieldFalloffType::Field_FallOff_None>(const float& MinRange, const float& DeltaRange, const float& NodeMagnitude, const float& FalloffValue)
{
	return ScaleFunctionResult(MinRange, DeltaRange, NodeMagnitude, 1.0);
}

template<>
float EvalFalloffFunction<EFieldFalloffType::Field_Falloff_Linear>(const float& MinRange, const float& DeltaRange, const float& NodeMagnitude, const float& FalloffValue)
{
	return ScaleFunctionResult(MinRange, DeltaRange, NodeMagnitude, FalloffValue);
}

template<>
float EvalFalloffFunction<EFieldFalloffType::Field_Falloff_Squared>(const float& MinRange, const float& DeltaRange, const float& NodeMagnitude, const float& FalloffValue)
{
	return ScaleFunctionResult(MinRange, DeltaRange, NodeMagnitude, FalloffValue * FalloffValue);
}

template<>
float EvalFalloffFunction<EFieldFalloffType::Field_Falloff_Inverse>(const float& MinRange, const float& DeltaRange, const float& NodeMagnitude, const float& FalloffValue)
{
	return ScaleFunctionResult(MinRange, DeltaRange, NodeMagnitude, 2.0f * (1.0f - 1.0f / (FalloffValue + 1.0f)));
}

template<>
float EvalFalloffFunction<EFieldFalloffType::Field_Falloff_Logarithmic>(const float& MinRange, const float& DeltaRange, const float& NodeMagnitude, const float& FalloffValue)
{
	return ScaleFunctionResult(MinRange, DeltaRange, NodeMagnitude, FMath::LogX(2.0f, FalloffValue + 1.0f));
}

/**
* FRadialFalloff
*/

template<EFieldFalloffType FalloffType>
void FRadialFalloff::Evaluator(const FFieldContext& Context, TFieldArrayView<float>& Results) const  
{
	if (Radius > 0.f)
	{
		const float DeltaRange = (MaxRange - MinRange);
		const int32 NumSamples = Context.SampleIndices.Num();

		for (int32 SampleIndex = 0; SampleIndex < NumSamples; SampleIndex++)
		{
			const FFieldContextIndex& Index = Context.SampleIndices[SampleIndex];
			{
				Results[Index.Result] = Default;
				const FVector::FReal Delta = (Context.SamplePositions[Index.Sample] - Position).Size();

				if (Delta < Radius)
				{
					const float Function = float(1.0f - Delta / Radius);		// LWC_TODO: Precision loss
					Results[Index.Result] = EvalFalloffFunction<FalloffType>(MinRange, DeltaRange, Magnitude, Function);
				}
			}
		}
	}
}

void FRadialFalloff::Evaluate(FFieldContext& Context, TFieldArrayView<float>& Results) const
{
	switch (Falloff)
	{
	case EFieldFalloffType::Field_FallOff_None:
		Evaluator<EFieldFalloffType::Field_FallOff_None>(Context, Results);
		break;
	case EFieldFalloffType::Field_Falloff_Linear:
		Evaluator<EFieldFalloffType::Field_Falloff_Linear>(Context, Results);
		break;
	case EFieldFalloffType::Field_Falloff_Squared:
		Evaluator<EFieldFalloffType::Field_Falloff_Squared>(Context, Results);
		break;
	case EFieldFalloffType::Field_Falloff_Inverse:
		Evaluator<EFieldFalloffType::Field_Falloff_Inverse>(Context, Results);
		break;
	case EFieldFalloffType::Field_Falloff_Logarithmic:
		Evaluator<EFieldFalloffType::Field_Falloff_Logarithmic>(Context,  Results);
		break;
	}
}

void FRadialFalloff::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar << Magnitude;
	Ar << MinRange;
	Ar << MaxRange;
	Ar << Default;
	Ar << Radius;
	Ar << Position;
	SerializeInternal<EFieldFalloffType>(Ar, Falloff);

}

bool FRadialFalloff::operator==(const FFieldNodeBase& Node)
{
	if (Node.SerializationType() == SerializationType())
	{
		const FRadialFalloff* Other = static_cast<const FRadialFalloff*>(&Node);
		return Super::operator==(Node)
			&& Magnitude == Other->Magnitude
			&& MinRange == Other->MinRange
			&& MaxRange == Other->MaxRange
			&& Default == Other->Default
			&& Radius == Other->Radius
			&& Position == Other->Position
			&& Falloff == Other->Falloff;
	}
	return false;
}

void FRadialFalloff::FillSetupCount(int32& NumOffsets, int32& NumParams) const
{
	Super::FillSetupCount(NumOffsets, NumParams);
	NumParams += 9;
}

void FRadialFalloff::FillSetupDatas(TArray<int32>& NodesOffsets, TArray<float>& NodesParams, const float CommandTime) const
{
	Super::FillSetupDatas(NodesOffsets, NodesParams, CommandTime);
	NodesParams.Add(Magnitude);
	NodesParams.Add(MinRange);
	NodesParams.Add(MaxRange);
	NodesParams.Add(Default);
	NodesParams.Add(Radius);
	NodesParams.Add(static_cast<float>(Position.X));
	NodesParams.Add(static_cast<float>(Position.Y));
	NodesParams.Add(static_cast<float>(Position.Z));
	NodesParams.Add(static_cast<float>(Falloff));
}

float FRadialFalloff::EvalMaxMagnitude() const
{
	return Magnitude;
}

void FRadialFalloff::ComputeFieldBounds(FVector& MinBounds, FVector& MaxBounds, FVector& CenterPosition) const
{
	MinBounds = (Default == 0) ? Position - FVector(Radius) : FVector(-FLT_MAX);
	MaxBounds = (Default == 0) ? Position + FVector(Radius) : FVector(FLT_MAX);
	CenterPosition = Position;
}

/**
* FPlaneFalloff
*/

template<EFieldFalloffType FalloffType>
void FPlaneFalloff::Evaluator(const FFieldContext& Context, const FPlane& Plane, TFieldArrayView<float>& Results) const
{
	if (Distance > 0.f)
	{
		const float DeltaRange = (MaxRange - MinRange);
		const int32 NumSamples = Context.SampleIndices.Num();
		for (int32 SampleIndex = 0; SampleIndex < NumSamples; SampleIndex++)
		{
			const FFieldContextIndex& Index = Context.SampleIndices[SampleIndex];
			{
				Results[Index.Result] = Default;
				const FPlane::FReal Delta = Plane.PlaneDot(Context.SamplePositions[Index.Sample]);

				if (Delta < -UE_SMALL_NUMBER && Delta > -Distance)
				{
					const float Function = float(1.0f + Delta / Distance);		// LWC_TODO: Precision loss
					Results[Index.Result] = EvalFalloffFunction<FalloffType>(MinRange, DeltaRange, Magnitude, Function);
				}
			}
		}
	}
}

void
FPlaneFalloff::Evaluate(FFieldContext& Context, TFieldArrayView<float>& Results) const
{
	FPlane Plane(Position, Normal);
	switch (Falloff)
	{
	case EFieldFalloffType::Field_FallOff_None:
		Evaluator<EFieldFalloffType::Field_FallOff_None>(Context, Plane, Results);
		break;
	case EFieldFalloffType::Field_Falloff_Linear:
		Evaluator<EFieldFalloffType::Field_Falloff_Linear>(Context, Plane, Results);
		break;
	case EFieldFalloffType::Field_Falloff_Squared:
		Evaluator<EFieldFalloffType::Field_Falloff_Squared>(Context, Plane, Results);
		break;
	case EFieldFalloffType::Field_Falloff_Inverse:
		Evaluator<EFieldFalloffType::Field_Falloff_Inverse>(Context, Plane, Results);
		break;
	case EFieldFalloffType::Field_Falloff_Logarithmic:
		Evaluator<EFieldFalloffType::Field_Falloff_Logarithmic>(Context, Plane, Results);
		break;
	}
}
void FPlaneFalloff::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar << Magnitude;
	Ar << MinRange;
	Ar << MaxRange;
	Ar << Default;
	Ar << Distance;
	Ar << Position;
	Ar << Normal;
	SerializeInternal<EFieldFalloffType>(Ar, Falloff);

}
bool FPlaneFalloff::operator==(const FFieldNodeBase& Node)
{
	if (Node.SerializationType() == SerializationType())
	{
		const FPlaneFalloff* Other = static_cast<const FPlaneFalloff*>(&Node);
		return Super::operator==(Node)
			&& Magnitude == Other->Magnitude
			&& MinRange == Other->MinRange
			&& MaxRange == Other->MaxRange
			&& Default == Other->Default
			&& Distance == Other->Distance
			&& Position == Other->Position
			&& Normal == Other->Normal
			&& Falloff == Other->Falloff;
	}
	return false;
}

void FPlaneFalloff::FillSetupCount(int32& NumOffsets, int32& NumParams) const
{
	Super::FillSetupCount(NumOffsets, NumParams);
	NumParams += 12;
}

void FPlaneFalloff::FillSetupDatas(TArray<int32>& NodesOffsets, TArray<float>& NodesParams, const float CommandTime) const
{
	Super::FillSetupDatas(NodesOffsets, NodesParams, CommandTime);
	NodesParams.Add(Magnitude);
	NodesParams.Add(MinRange);
	NodesParams.Add(MaxRange);
	NodesParams.Add(Default);
	NodesParams.Add(Distance);
	NodesParams.Add(static_cast<float>(Position.X));
	NodesParams.Add(static_cast<float>(Position.Y));
	NodesParams.Add(static_cast<float>(Position.Z));
	NodesParams.Add(static_cast<float>(Normal.X));
	NodesParams.Add(static_cast<float>(Normal.Y));
	NodesParams.Add(static_cast<float>(Normal.Z));
	NodesParams.Add(static_cast<float>(Falloff));
}

float FPlaneFalloff::EvalMaxMagnitude() const
{
	return Magnitude;
}

void FPlaneFalloff::ComputeFieldBounds(FVector& MinBounds, FVector& MaxBounds, FVector& CenterPosition) const
{
	MinBounds = FVector(-FLT_MAX);
	MaxBounds = FVector(FLT_MAX);
	CenterPosition = Position;
}

/**
* FBoxFalloff
*/

template<EFieldFalloffType FalloffType>
void FBoxFalloff::Evaluator(const FFieldContext& Context, TFieldArrayView<float>& Results) const
{
	const float DeltaRange = (MaxRange - MinRange);

	static const float HalfBox = 50.0;
	static const FBox UnitBox(FVector(-HalfBox), FVector(HalfBox));

	const int32 NumSamples = Context.SampleIndices.Num();
	for (int32 SampleIndex = 0; SampleIndex < NumSamples; SampleIndex++)
	{
		const FFieldContextIndex& Index = Context.SampleIndices[SampleIndex];
		{
			Results[Index.Result] = Default;
			const FVector LocalPoint = Transform.InverseTransformPosition(Context.SamplePositions[Index.Sample]);
			if (UnitBox.IsInside(LocalPoint))
			{
				const FVector Distance(FMath::Abs(LocalPoint.X)- HalfBox, FMath::Abs(LocalPoint.Y) - HalfBox, FMath::Abs(LocalPoint.Z) - HalfBox);
				const float Delta = (float)FMath::Min(FMath::Max(Distance.X, FMath::Max(Distance.Y, Distance.Z)), 0.0f);	// LWC_TODO: Precision loss
				const float Function = - Delta / HalfBox;

				Results[Index.Result] = EvalFalloffFunction<FalloffType>(MinRange, DeltaRange, Magnitude, Function);
			}
		}
	}
}

void FBoxFalloff::Evaluate(FFieldContext& Context, TFieldArrayView<float>& Results) const
{
	switch (Falloff)
	{
	case EFieldFalloffType::Field_FallOff_None:
		Evaluator<EFieldFalloffType::Field_FallOff_None>(Context, Results);
		break;
	case EFieldFalloffType::Field_Falloff_Linear:
		Evaluator<EFieldFalloffType::Field_Falloff_Linear>(Context, Results);
		break;
	case EFieldFalloffType::Field_Falloff_Squared:
		Evaluator<EFieldFalloffType::Field_Falloff_Squared>(Context, Results);
		break;
	case EFieldFalloffType::Field_Falloff_Inverse:
		Evaluator<EFieldFalloffType::Field_Falloff_Inverse>(Context, Results);
		break;
	case EFieldFalloffType::Field_Falloff_Logarithmic:
		Evaluator<EFieldFalloffType::Field_Falloff_Logarithmic>(Context, Results);
		break;
	}
}

void FBoxFalloff::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar << Magnitude;
	Ar << MinRange;
	Ar << MaxRange;
	Ar << Default;
	Ar << Transform;
	SerializeInternal<EFieldFalloffType>(Ar, Falloff);
}

bool FBoxFalloff::operator==(const FFieldNodeBase& Node)
{
	if (Node.SerializationType() == SerializationType())
	{
		const FBoxFalloff* Other = static_cast<const FBoxFalloff*>(&Node);
		bool bRet = Super::operator==(Node);
		bRet &= Magnitude == Other->Magnitude;
		bRet &= MinRange == Other->MinRange;
		bRet &= MaxRange == Other->MaxRange;
		bRet &= Default == Other->Default;
		bRet &= Transform.Equals(Other->Transform);
		bRet &= Falloff == Other->Falloff;
		return bRet;
	}
	return false;
}

void FBoxFalloff::FillSetupCount(int32& NumOffsets, int32& NumParams) const
{
	Super::FillSetupCount(NumOffsets, NumParams);
	NumParams += 15;
}

void FBoxFalloff::FillSetupDatas(TArray<int32>& NodesOffsets, TArray<float>& NodesParams, const float CommandTime) const
{
	Super::FillSetupDatas(NodesOffsets, NodesParams, CommandTime);
	NodesParams.Add(Magnitude);
	NodesParams.Add(MinRange);
	NodesParams.Add(MaxRange);
	NodesParams.Add(Default);
	NodesParams.Add(static_cast<float>(Transform.GetRotation().X));
	NodesParams.Add(static_cast<float>(Transform.GetRotation().Y));
	NodesParams.Add(static_cast<float>(Transform.GetRotation().Z));
	NodesParams.Add(static_cast<float>(Transform.GetRotation().W));
	NodesParams.Add(static_cast<float>(Transform.GetTranslation().X));
	NodesParams.Add(static_cast<float>(Transform.GetTranslation().Y));
	NodesParams.Add(static_cast<float>(Transform.GetTranslation().Z));
	NodesParams.Add(static_cast<float>(Transform.GetScale3D().X));
	NodesParams.Add(static_cast<float>(Transform.GetScale3D().Y));
	NodesParams.Add(static_cast<float>(Transform.GetScale3D().Z));
	NodesParams.Add(static_cast<float>(Falloff));
}

float FBoxFalloff::EvalMaxMagnitude() const
{
	return Magnitude;
}

void FBoxFalloff::ComputeFieldBounds(FVector& MinBounds, FVector& MaxBounds, FVector& CenterPosition) const
{
	MinBounds = FVector(-FLT_MAX);
	MaxBounds = FVector(FLT_MAX);
	if (Default == 0)
	{
		const FBox UnitBox(FVector(-50), FVector(50));
		const FBox BoundingBox = UnitBox.TransformBy(Transform);
		MinBounds = BoundingBox.Min;
		MaxBounds = BoundingBox.Max;
	}
	CenterPosition = Transform.GetTranslation();
}

/**
* FNoiseField
*/

void FNoiseField::Evaluate(FFieldContext& Context, TFieldArrayView<float>& Results) const
{
	const float DeltaRange = (MaxRange - MinRange);
	const int32 NumSamples = Context.SampleIndices.Num();

	for (int32 SampleIndex = 0; SampleIndex < NumSamples; SampleIndex++)
	{
		const FFieldContextIndex& Index = Context.SampleIndices[SampleIndex];

		FVector::FReal Dummy = 0.0f;
		FVector LocalPoint = Transform.InverseTransformPosition(Context.SamplePositions[Index.Sample]);
		LocalPoint = FVector(FMath::Modf(LocalPoint.X, &Dummy) * 0.5f + 0.5f, 
							 FMath::Modf(LocalPoint.Y, &Dummy) * 0.5f + 0.5f,
							 FMath::Modf(LocalPoint.Z, &Dummy) * 0.5f + 0.5f) * 255;

		// Samples for the Perlin noise must be btw 0->255
		float PerlinValue = 0.0f;
		Field::PerlinNoise::Sample(&PerlinValue, (float)LocalPoint.X, (float)LocalPoint.Y, (float)LocalPoint.Z);

		// Perlin noise result is btw -1 -> 1
		PerlinValue = 0.5f * ( PerlinValue + 1.0f );
		Results[Index.Result] = ScaleFunctionResult(MinRange, DeltaRange, 1.0f, PerlinValue);
	}
}

void FNoiseField::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar << MinRange;
	Ar << MaxRange;
	Ar << Transform;
}

bool FNoiseField::operator==(const FFieldNodeBase& Node)
{
	if (Node.SerializationType() == SerializationType())
	{
		const FNoiseField* Other = static_cast<const FNoiseField*>(&Node);
		return Super::operator==(Node)
			&& MinRange == Other->MinRange
			&& MaxRange == Other->MaxRange
			&& Transform.Equals(Other->Transform);
	}
	return false;
}

void FNoiseField::FillSetupCount(int32& NumOffsets, int32& NumParams) const
{
	Super::FillSetupCount(NumOffsets, NumParams);
	NumParams += 12;
}

void FNoiseField::FillSetupDatas(TArray<int32>& NodesOffsets, TArray<float>& NodesParams, const float CommandTime) const
{
	Super::FillSetupDatas(NodesOffsets, NodesParams, CommandTime);
	NodesParams.Add(MinRange);
	NodesParams.Add(MaxRange);
	NodesParams.Add(static_cast<float>(Transform.GetRotation().X));
	NodesParams.Add(static_cast<float>(Transform.GetRotation().Y));
	NodesParams.Add(static_cast<float>(Transform.GetRotation().Z));
	NodesParams.Add(static_cast<float>(Transform.GetRotation().W));
	NodesParams.Add(static_cast<float>(Transform.GetTranslation().X));
	NodesParams.Add(static_cast<float>(Transform.GetTranslation().Y));
	NodesParams.Add(static_cast<float>(Transform.GetTranslation().Z));
	NodesParams.Add(static_cast<float>(Transform.GetScale3D().X));
	NodesParams.Add(static_cast<float>(Transform.GetScale3D().Y));
	NodesParams.Add(static_cast<float>(Transform.GetScale3D().Z));
}

float FNoiseField::EvalMaxMagnitude() const
{
	return MaxRange;
}

/**
* FUniformVector
*/

void FUniformVector::Evaluate(FFieldContext& Context, TFieldArrayView<FVector>& Results) const
{
	const FVector DirectionVal = Direction;
	const float MagnitudeVal = Magnitude;

	const int32 NumSamples = Context.SampleIndices.Num();
	for (int32 SampleIndex = 0; SampleIndex < NumSamples; SampleIndex++)
	{
		Results[Context.SampleIndices[SampleIndex].Result] = MagnitudeVal * DirectionVal;
	}
}

void FUniformVector::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar << Magnitude;
	Ar << Direction;
}

bool FUniformVector::operator==(const FFieldNodeBase& Node)
{
	if (Node.SerializationType() == SerializationType())
	{
		const FUniformVector* Other = static_cast<const FUniformVector*>(&Node);
		return Super::operator==(Node)
			&& Magnitude == Other->Magnitude
			&& Direction == Other->Direction;
	}
	return false;
}

void FUniformVector::FillSetupCount(int32& NumOffsets, int32& NumParams) const
{
	Super::FillSetupCount(NumOffsets, NumParams);
	NumParams += 4;
}

void FUniformVector::FillSetupDatas(TArray<int32>& NodesOffsets, TArray<float>& NodesParams, const float CommandTime) const 
{
	Super::FillSetupDatas(NodesOffsets, NodesParams, CommandTime);
	NodesParams.Add(Magnitude);
	NodesParams.Add(static_cast<float>(Direction.X));
	NodesParams.Add(static_cast<float>(Direction.Y));
	NodesParams.Add(static_cast<float>(Direction.Z));
}

float FUniformVector::EvalMaxMagnitude() const
{
	return Magnitude;
}

/**
* FRadialVector
*/

void FRadialVector::Evaluate(FFieldContext& Context, TFieldArrayView<FVector>& Results) const
{
	const int32 NumSamples = Context.SampleIndices.Num();
	for (int32 SampleIndex = 0; SampleIndex < NumSamples; SampleIndex++)
	{
		const FFieldContextIndex& Index = Context.SampleIndices[SampleIndex];
		{
			Results[Index.Result] = Magnitude * (Context.SamplePositions[Index.Sample] - Position).GetSafeNormal();
		}
	}
	
}

void FRadialVector::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar << Magnitude;
	Ar << Position;
}

bool FRadialVector::operator==(const FFieldNodeBase& Node)
{
	if (Node.SerializationType() == SerializationType())
	{
		const FRadialVector* Other = static_cast<const FRadialVector*>(&Node);
		return Super::operator==(Node)
			&& Magnitude == Other->Magnitude
			&& Position == Other->Position;
	}
	return false;
}

void FRadialVector::FillSetupCount(int32& NumOffsets, int32& NumParams) const
{
	Super::FillSetupCount(NumOffsets, NumParams);
	NumParams += 4;
}

void FRadialVector::FillSetupDatas(TArray<int32>& NodesOffsets, TArray<float>& NodesParams, const float CommandTime) const
{
	Super::FillSetupDatas(NodesOffsets, NodesParams, CommandTime);
	NodesParams.Add(Magnitude);
	NodesParams.Add(static_cast<float>(Position.X));
	NodesParams.Add(static_cast<float>(Position.Y));
	NodesParams.Add(static_cast<float>(Position.Z));
}

float FRadialVector::EvalMaxMagnitude() const
{
	return Magnitude;
}

void FRadialVector::ComputeFieldBounds(FVector& MinBounds, FVector& MaxBounds, FVector& CenterPosition) const
{
	MinBounds = FVector(-FLT_MAX);
	MaxBounds = FVector(FLT_MAX);
	CenterPosition = Position;
}


/**
* FRotatedRadialVector
*/

void FRotatedRadialVector::Evaluate(FFieldContext& Context, TFieldArrayView<FVector>& Results) const
{
	const FQuat Rotation = FQuat(Axis.GetSafeNormal(), FMath::DegreesToRadians(Angle));
	const int32 NumSamples = Context.SampleIndices.Num();
	for (int32 SampleIndex = 0; SampleIndex < NumSamples; SampleIndex++)
	{
		const FFieldContextIndex& Index = Context.SampleIndices[SampleIndex];
		{
			const FVector RadialDir = (Context.SamplePositions[Index.Sample] - Position).GetSafeNormal();
			Results[Index.Result] = Magnitude * Rotation.RotateVector(RadialDir);
		}
	}
}

void FRotatedRadialVector::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar << Magnitude;
	Ar << Position;
	Ar << Angle;
	Ar << Axis;
}

bool FRotatedRadialVector::operator==(const FFieldNodeBase& Node)
{
	if (Node.SerializationType() == SerializationType())
	{
		const FRotatedRadialVector* Other = static_cast<const FRotatedRadialVector*>(&Node);
		return Super::operator==(Node)
			&& Magnitude == Other->Magnitude
			&& Position == Other->Position
			&& Angle == Other->Angle
			&& Axis == Other->Axis;
	}
	return false;
}

void FRotatedRadialVector::FillSetupCount(int32& NumOffsets, int32& NumParams) const
{
	Super::FillSetupCount(NumOffsets, NumParams);
	NumParams += 8; // Magnitude + Position(3) + Angle + Axis(3)
}

void FRotatedRadialVector::FillSetupDatas(TArray<int32>& NodesOffsets, TArray<float>& NodesParams, const float CommandTime) const
{
	Super::FillSetupDatas(NodesOffsets, NodesParams, CommandTime);
	NodesParams.Add(Magnitude);
	NodesParams.Add(static_cast<float>(Position.X));
	NodesParams.Add(static_cast<float>(Position.Y));
	NodesParams.Add(static_cast<float>(Position.Z));
	NodesParams.Add(Angle);
	NodesParams.Add(static_cast<float>(Axis.X));
	NodesParams.Add(static_cast<float>(Axis.Y));
	NodesParams.Add(static_cast<float>(Axis.Z));
}

float FRotatedRadialVector::EvalMaxMagnitude() const
{
	return Magnitude;
}

void FRotatedRadialVector::ComputeFieldBounds(FVector& MinBounds, FVector& MaxBounds, FVector& CenterPosition) const
{
	MinBounds = FVector(-FLT_MAX);
	MaxBounds = FVector(FLT_MAX);
	CenterPosition = Position;
}

/**
* FRandomVector
*/

void FRandomVector::Evaluate(FFieldContext& Context, TFieldArrayView<FVector>& Results) const
{
	const int32 NumSamples = Context.SampleIndices.Num();
	for (int32 SampleIndex = 0; SampleIndex < NumSamples; SampleIndex++)
	{
		const FFieldContextIndex& Index = Context.SampleIndices[SampleIndex];
		{
			Results[Index.Result].X = FMath::RandRange(-1.f, 1.f);
			Results[Index.Result].Y = FMath::RandRange(-1.f, 1.f);
			Results[Index.Result].Z = FMath::RandRange(-1.f, 1.f);
			Results[Index.Result].Normalize();
			Results[Index.Result] *= Magnitude;
		}
	}
}

void FRandomVector::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar << Magnitude;
}

bool FRandomVector::operator==(const FFieldNodeBase& Node)
{
	if (Node.SerializationType() == SerializationType())
	{
		return Super::operator==(Node)
			&& Magnitude == static_cast<const FRandomVector*>(&Node)->Magnitude;
	}
	return false;
}

void FRandomVector::FillSetupCount(int32& NumOffsets, int32& NumParams) const 
{
	Super::FillSetupCount(NumOffsets, NumParams);
	NumParams += 1;
}

void FRandomVector::FillSetupDatas(TArray<int32>& NodesOffsets, TArray<float>& NodesParams, const float CommandTime) const 
{
	Super::FillSetupDatas(NodesOffsets, NodesParams, CommandTime);
	NodesParams.Add(Magnitude);
}

float FRandomVector::EvalMaxMagnitude() const 
{
	return Magnitude;
}


/**
* FSumScalar 
* 
*/

void FSumScalar::Evaluate(FFieldContext& ContextIn, TFieldArrayView<float>& Results) const
{
	TUniquePtr<FFieldSystemMetaDataResults<float> > ResultsData(new FFieldSystemMetaDataResults<float>(Results));
	FScopedFieldContextMetaData ScopedMetaData(ContextIn, ResultsData.Get());

	const int32 NumResults = Results.Num();
	const int32 NumSamples = ContextIn.SampleIndices.Num();

	const FFieldNode<float> * RightField = ScalarRight.Get();
	const FFieldNode<float> * LeftField = ScalarLeft.Get();

	float MagnitudeVal = Magnitude;
	if (LeftField != nullptr && RightField != nullptr)
	{ 
		const uint32 BufferOffset = ContextIn.ScalarResults.Num();
		ContextIn.ScalarResults.SetNum(BufferOffset + 2* NumResults, EAllowShrinking::No);

		TFieldArrayView<float> Buffers[2] = {
			TFieldArrayView<float>(ContextIn.ScalarResults,BufferOffset,NumResults),
			TFieldArrayView<float>(ContextIn.ScalarResults,BufferOffset+NumResults,NumResults),
		};

		TArray<const FFieldNode<float> * > FieldNodes = { LeftField,RightField };

		for (int32 i = 0; i < 2; i++)
		{
			if (ensureMsgf(FieldNodes[i]->Type() == FFieldNode<float>::StaticType(),
				TEXT("Field system SumScalar expects float input arrays.")))
			{
				FieldNodes[i]->Evaluate(ContextIn, Buffers[i]);
			}
		}

		switch (Operation)
		{
		case EFieldOperationType::Field_Multiply:
			for (int32 SampleIndex = 0; SampleIndex < NumSamples; SampleIndex++)
			{
				const FFieldContextIndex& Index = ContextIn.SampleIndices[SampleIndex];
				{
					Results[Index.Result] = Buffers[1][Index.Result] * Buffers[0][Index.Result];
				}
			}
			break;
		case EFieldOperationType::Field_Divide:
			for (int32 SampleIndex = 0; SampleIndex < NumSamples; SampleIndex++)
			{
				const FFieldContextIndex& Index = ContextIn.SampleIndices[SampleIndex];
				{
					Results[Index.Result] = Buffers[0][Index.Result] / Buffers[1][Index.Result];
				}
			}
			break;
		case EFieldOperationType::Field_Add:
			for (int32 SampleIndex = 0; SampleIndex < NumSamples; SampleIndex++)
			{
				const FFieldContextIndex& Index = ContextIn.SampleIndices[SampleIndex];
				{
					Results[Index.Result] = Buffers[1][Index.Result] + Buffers[0][Index.Result];
				}
			}
			break;
		case EFieldOperationType::Field_Substract:
			for (int32 SampleIndex = 0; SampleIndex < NumSamples; SampleIndex++)
			{
				const FFieldContextIndex& Index = ContextIn.SampleIndices[SampleIndex];
				{
					Results[Index.Result] = Buffers[0][Index.Result] - Buffers[1][Index.Result];
				}
			}
			break;
		}
		ContextIn.ScalarResults.SetNum(BufferOffset, EAllowShrinking::No);
	}
	else if (LeftField != nullptr && ensureMsgf(ScalarLeft->Type() == FFieldNode<float>::StaticType(),
		TEXT("Field system SumScalar expects float input arrays.")))
	{
		LeftField->Evaluate(ContextIn, Results);

	}
	else if (ScalarRight != nullptr &&  ensureMsgf(ScalarRight->Type() == FFieldNode<float>::StaticType(),
		TEXT("Field system SumScalar expects float input arrays.")))
	{
		ScalarRight->Evaluate(ContextIn, Results);
	}

	if (MagnitudeVal != 1.0)
	{
		for (int32 SampleIndex = 0; SampleIndex < NumSamples; SampleIndex++)
		{
			const FFieldContextIndex& Index = ContextIn.SampleIndices[SampleIndex];
			{
				Results[Index.Result] *= MagnitudeVal;
			}
		}
	}
}

void FSumScalar::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar << Magnitude;
	SerializeInternal<float>(Ar, ScalarRight);
	SerializeInternal<float>(Ar, ScalarLeft);
	SerializeInternal<EFieldOperationType>(Ar, Operation);
}

bool FSumScalar::operator==(const FFieldNodeBase& Node)
{
	if (Node.SerializationType() == SerializationType())
	{
		const FSumScalar* Other = static_cast<const FSumScalar*>(&Node);
		bool bRet = Super::operator==(Node);
		bRet &= Magnitude == Other->Magnitude;
		bRet &= FieldsEqual(ScalarRight,Other->ScalarRight);
		bRet &= FieldsEqual(ScalarLeft,Other->ScalarLeft);
		bRet &= Operation == Other->Operation;
		return bRet;
	}
	return false;
}

void FSumScalar::FillSetupCount(int32& NumOffsets, int32& NumParams) const
{
	if (ScalarRight.IsValid())
	{
		ScalarRight->FillSetupCount(NumOffsets, NumParams);
	}
	if (ScalarLeft.IsValid())
	{
		ScalarLeft->FillSetupCount(NumOffsets, NumParams);
	}
	Super::FillSetupCount(NumOffsets, NumParams);
	NumParams += 4;
}

void FSumScalar::FillSetupDatas(TArray<int32>& NodesOffsets, TArray<float>& NodesParams, const float CommandTime) const
{
	if (ScalarRight.IsValid())
	{
		ScalarRight->FillSetupDatas(NodesOffsets, NodesParams, CommandTime);
	}
	if (ScalarLeft.IsValid())
	{
		ScalarLeft->FillSetupDatas(NodesOffsets, NodesParams, CommandTime);
	}
	Super::FillSetupDatas(NodesOffsets, NodesParams, CommandTime);
	NodesParams.Add(Magnitude);
	NodesParams.Add(static_cast<float>(ScalarRight != nullptr));
	NodesParams.Add(static_cast<float>(ScalarLeft != nullptr));
	NodesParams.Add(static_cast<float>(Operation));
}

float FSumScalar::EvalMaxMagnitude() const
{
	float MaxMagnitudeA = 0.0, MaxMagnitudeB = 0.0, MaxMagnitude = 0.0;
	if (ScalarRight.IsValid())
	{
		MaxMagnitudeA = ScalarRight->EvalMaxMagnitude();
	}
	if (ScalarLeft.IsValid())
	{
		MaxMagnitudeB = ScalarLeft->EvalMaxMagnitude();
	}
	if (Operation == EFieldOperationType::Field_Multiply ||
		Operation == EFieldOperationType::Field_Divide)
	{
		MaxMagnitude = (Operation == EFieldOperationType::Field_Multiply) ? MaxMagnitudeA * MaxMagnitudeB :
			(FMath::Abs(MaxMagnitudeA) > FLT_EPSILON) ? MaxMagnitudeB / MaxMagnitudeA : 0.0f;
	}
	else if (Operation == EFieldOperationType::Field_Add ||
		Operation == EFieldOperationType::Field_Substract)
	{
		MaxMagnitude = FMath::Max(MaxMagnitudeA, MaxMagnitudeB);
	}

	return Magnitude * MaxMagnitude;
}

void FSumScalar::ComputeFieldBounds(FVector& MinBounds, FVector& MaxBounds, FVector& CenterPosition) const
{
	FVector MinBoundsA(-FLT_MAX), MaxBoundsA(FLT_MAX), MinBoundsB(-FLT_MAX), MaxBoundsB(FLT_MAX);
	FVector CenterPositionA = FVector::Zero(), CenterPositionB = FVector::Zero();
	if (ScalarRight.IsValid())
	{
		ScalarRight->ComputeFieldBounds(MinBoundsA, MaxBoundsA, CenterPositionA);
	}
	if (ScalarLeft.IsValid())
	{
		ScalarLeft->ComputeFieldBounds(MinBoundsB, MaxBoundsB, CenterPositionB);
	}
	CenterPosition = FVector::Zero();
	int32 NumCenters = 0;
	if (CenterPositionA != FVector::Zero())
	{
		CenterPosition += CenterPositionA;
		++NumCenters;
	}
	if (CenterPositionB != FVector::Zero())
	{
		CenterPosition += CenterPositionB;
		++NumCenters;
	}
	CenterPosition = (NumCenters > 0) ? CenterPosition / NumCenters : FVector::Zero();

	if (Operation == EFieldOperationType::Field_Multiply ||
		Operation == EFieldOperationType::Field_Divide)
	{
		MinBounds = MaxVector(MinBoundsA, MinBoundsB);
		MaxBounds = MinVector(MaxBoundsA, MaxBoundsB);
	}
	else if (Operation == EFieldOperationType::Field_Add ||
		Operation == EFieldOperationType::Field_Substract)
	{
		MinBounds = MinVector(MinBoundsA, MinBoundsB);
		MaxBounds = MaxVector(MaxBoundsA, MaxBoundsB);
	}
}

/**
* FSumVector
*/
void FSumVector::Evaluate(FFieldContext& ContextIn, TFieldArrayView<FVector>& Results) const
{
	TUniquePtr<FFieldSystemMetaDataResults<FVector>> ResultsData(new FFieldSystemMetaDataResults<FVector>(Results));
	FScopedFieldContextMetaData ScopedMetaData(ContextIn, ResultsData.Get());

	int32 NumResults = Results.Num();
	int32 NumSamples = ContextIn.SampleIndices.Num();

	const FFieldNode<float> * ScalarField = Scalar.Get();
	const FFieldNode<FVector> * RightVectorField = VectorRight.Get();
	const FFieldNode<FVector> * LeftVectorField = VectorLeft.Get();

	float MagnitudeVal = Magnitude;
	if (RightVectorField != nullptr && LeftVectorField != nullptr)
	{
		const uint32 BufferOffset = ContextIn.VectorResults.Num();
		ContextIn.VectorResults.SetNum(BufferOffset + 2 * NumResults, EAllowShrinking::No);

		TFieldArrayView<FVector> Buffers[2] = {
			TFieldArrayView<FVector>(ContextIn.VectorResults, BufferOffset, NumResults),
			TFieldArrayView<FVector>(ContextIn.VectorResults, BufferOffset + NumResults, NumResults),
		};

		LeftVectorField->Evaluate(ContextIn, Buffers[0]);
		RightVectorField->Evaluate(ContextIn, Buffers[1]);

		switch (Operation)
		{
		case EFieldOperationType::Field_Multiply:
			for (int32 SampleIndex = 0; SampleIndex < NumSamples; SampleIndex++)
			{
				const FFieldContextIndex& Index = ContextIn.SampleIndices[SampleIndex];
				{
					Results[Index.Result] = Buffers[1][Index.Result] * Buffers[0][Index.Result];
				}
			}
			break;
		case EFieldOperationType::Field_Divide:
			for (int32 SampleIndex = 0; SampleIndex < NumSamples; SampleIndex++)
			{
		  const FFieldContextIndex& Index = ContextIn.SampleIndices[SampleIndex];
				{
					Results[Index.Result] = Buffers[0][Index.Result] / Buffers[1][Index.Result];
				}
			}
			break;
		case EFieldOperationType::Field_Add:
			for (int32 SampleIndex = 0; SampleIndex < NumSamples; SampleIndex++)
			{
		  const FFieldContextIndex& Index = ContextIn.SampleIndices[SampleIndex];
				{
					Results[Index.Result] = Buffers[1][Index.Result] + Buffers[0][Index.Result];
				}
			}
			break;
		case EFieldOperationType::Field_Substract:
			for (int32 SampleIndex = 0; SampleIndex < NumSamples; SampleIndex++)
			{
		  const FFieldContextIndex& Index = ContextIn.SampleIndices[SampleIndex];
				{
					Results[Index.Result] = Buffers[0][Index.Result] - Buffers[1][Index.Result];
				}
			}
			break;
		}
		ContextIn.VectorResults.SetNum(BufferOffset, EAllowShrinking::No);
	}
	else if (LeftVectorField != nullptr)
	{
		LeftVectorField->Evaluate(ContextIn, Results);
	}
	else if (RightVectorField != nullptr)
	{
		RightVectorField->Evaluate(ContextIn, Results);
	}

	if (ScalarField != nullptr)
	{
		const uint32 BufferOffset = ContextIn.ScalarResults.Num();
		ContextIn.ScalarResults.SetNum(BufferOffset + NumResults, EAllowShrinking::No);

		TFieldArrayView<float> Buffer(ContextIn.ScalarResults, BufferOffset, NumResults);

		ScalarField->Evaluate(ContextIn, Buffer);

		for (int32 SampleIndex = 0; SampleIndex < NumSamples; SampleIndex++)
		{
			 const FFieldContextIndex& Index = ContextIn.SampleIndices[SampleIndex];
			{
				Results[Index.Result] *= Buffer[Index.Result];
			}
		}
		ContextIn.ScalarResults.SetNum(BufferOffset, EAllowShrinking::No);
	}

	if (MagnitudeVal != 1.0)
	{
		for (int32 SampleIndex = 0; SampleIndex < NumSamples; SampleIndex++)
		{
			const FFieldContextIndex& Index = ContextIn.SampleIndices[SampleIndex];
			{
				Results[Index.Result] *= MagnitudeVal;
			}
		}
	}
}
void FSumVector::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar << Magnitude;
	SerializeInternal<float>(Ar, Scalar);
	SerializeInternal<FVector>(Ar, VectorRight);
	SerializeInternal<FVector>(Ar, VectorLeft);
	SerializeInternal<EFieldOperationType>(Ar, Operation);
}
bool FSumVector::operator==(const FFieldNodeBase& Node)
{
	if (Node.SerializationType() == SerializationType())
	{
		const FSumVector* Other = static_cast<const FSumVector*>(&Node);
		return Super::operator==(Node)
			&& Magnitude == Other->Magnitude
			&& FieldsEqual(Scalar,Other->Scalar)
			&& FieldsEqual(VectorRight, Other->VectorRight)
			&& FieldsEqual(VectorLeft, Other->VectorLeft)
			&& Operation == Other->Operation;
	}
	return false;
}

void FSumVector::FillSetupCount(int32& NumOffsets, int32& NumParams) const
{
	if (Scalar.IsValid())
	{
		Scalar->FillSetupCount(NumOffsets, NumParams);
	}
	if (VectorRight.IsValid())
	{
		VectorRight->FillSetupCount(NumOffsets, NumParams);
	}
	if (VectorLeft.IsValid())
	{
		VectorLeft->FillSetupCount(NumOffsets, NumParams);
	}
	Super::FillSetupCount(NumOffsets, NumParams);
	NumParams += 5;
}

void FSumVector::FillSetupDatas(TArray<int32>& NodesOffsets, TArray<float>& NodesParams, const float CommandTime) const
{
	if (Scalar.IsValid())
	{
		Scalar->FillSetupDatas(NodesOffsets, NodesParams, CommandTime);
	}
	if (VectorRight.IsValid())
	{
		VectorRight->FillSetupDatas(NodesOffsets, NodesParams, CommandTime);
	}
	if (VectorLeft.IsValid())
	{
		VectorLeft->FillSetupDatas(NodesOffsets, NodesParams, CommandTime);
	}
	Super::FillSetupDatas(NodesOffsets, NodesParams, CommandTime);
	NodesParams.Add(Magnitude);
	NodesParams.Add(static_cast<float>(Scalar != nullptr));
	NodesParams.Add(static_cast<float>(VectorRight != nullptr));
	NodesParams.Add(static_cast<float>(VectorLeft != nullptr));
	NodesParams.Add(static_cast<float>(Operation));
}

float FSumVector::EvalMaxMagnitude() const
{
	float MaxMagnitudeA = 0.0, MaxMagnitudeB = 0.0, MaxMagnitudeC = 0.0, MaxMagnitude = 0.0;
	if (Scalar.IsValid())
	{
		MaxMagnitudeA = Scalar->EvalMaxMagnitude();
	}
	if (VectorRight.IsValid())
	{
		MaxMagnitudeB = VectorRight->EvalMaxMagnitude();
	}
	if (VectorLeft.IsValid())
	{
		MaxMagnitudeC = VectorLeft->EvalMaxMagnitude();
	}
	if (Operation == EFieldOperationType::Field_Multiply ||
		Operation == EFieldOperationType::Field_Divide)
	{
		MaxMagnitude = (Operation == EFieldOperationType::Field_Multiply) ? MaxMagnitudeA * MaxMagnitudeB :
			(FMath::Abs(MaxMagnitudeA) > FLT_EPSILON) ? MaxMagnitudeB / MaxMagnitudeA : 0.0f;
	}
	else if (Operation == EFieldOperationType::Field_Add ||
		Operation == EFieldOperationType::Field_Substract)
	{
		MaxMagnitude = FMath::Max(MaxMagnitudeA, MaxMagnitudeB);
	}

	return Magnitude * MaxMagnitudeA * MaxMagnitude;
}

void FSumVector::ComputeFieldBounds(FVector& MinBounds, FVector& MaxBounds, FVector& CenterPosition) const
{
	FVector MinBoundsA(-FLT_MAX), MaxBoundsA(FLT_MAX), MinBoundsB(-FLT_MAX), MaxBoundsB(FLT_MAX), MinBoundsC(-FLT_MAX), MaxBoundsC(FLT_MAX);
	FVector CenterPositionA = FVector::Zero(), CenterPositionB = FVector::Zero(), CenterPositionC = FVector::Zero();
	if (Scalar.IsValid())
	{
		Scalar->ComputeFieldBounds(MinBoundsC, MaxBoundsC, CenterPositionC);
	}
	if (VectorRight.IsValid())
	{
		VectorRight->ComputeFieldBounds(MinBoundsA, MaxBoundsA, CenterPositionA);
	}
	if (VectorLeft.IsValid())
	{
		VectorLeft->ComputeFieldBounds(MinBoundsB, MaxBoundsB, CenterPositionB);
	}

	CenterPosition = FVector::Zero();
	int32 NumCenters = 0;
	if (CenterPositionA != FVector::Zero())
	{
		CenterPosition += CenterPositionA;
		++NumCenters;
	}
	if (CenterPositionB != FVector::Zero())
	{
		CenterPosition += CenterPositionB;
		++NumCenters;
	}
	if (CenterPositionC != FVector::Zero())
	{
		CenterPosition += CenterPositionC;
		++NumCenters;
	}
	CenterPosition = (NumCenters > 0) ? CenterPosition / NumCenters : FVector::Zero();

	if (Operation == EFieldOperationType::Field_Multiply ||
		Operation == EFieldOperationType::Field_Divide)
	{
		MinBounds = MaxVector(MinBoundsA, MinBoundsB);
		MaxBounds = MinVector(MaxBoundsA, MaxBoundsB);
	}
	else if (Operation == EFieldOperationType::Field_Add ||
		Operation == EFieldOperationType::Field_Substract)
	{
		MinBounds = MinVector(MinBoundsA, MinBoundsB);
		MaxBounds = MaxVector(MaxBoundsA, MaxBoundsB);
	}
	MinBounds = MaxVector(MinBounds, MinBoundsC);
	MaxBounds = MinVector(MaxBounds, MaxBoundsC);
}


/**
* FConversionField<InT,OutT>
*/

template<class InT, class OutT>
void FConversionField<InT,OutT>::Evaluate(FFieldContext& Context, TFieldArrayView<OutT>& Results) const
{
	static_assert(std::is_arithmetic_v<InT>, "Arithmetic types required for field conversion");
	static_assert(std::is_arithmetic_v<OutT>, "Arithmetic types required for field conversion");
	
	int32 NumResults = Results.Num();
	int32 NumSamples = Context.SampleIndices.Num();

	TArray<InT>& ResultsArray = GetResultArray<InT>(Context);

	const int32 BufferOffset = ResultsArray.Num();
	ResultsArray.SetNum(BufferOffset + NumResults, EAllowShrinking::No);
	FMemory::Memzero(&ResultsArray[BufferOffset], sizeof(InT) * NumResults);

	TFieldArrayView<InT> BufferView(ResultsArray, BufferOffset, NumResults);
	InputField->Evaluate(Context, BufferView);

	for (int32 SampleIndex = 0; SampleIndex < NumSamples; SampleIndex++)
	{
		const FFieldContextIndex& Index = Context.SampleIndices[SampleIndex];
		Results[Index.Result] = (OutT)BufferView[Index.Result];
	}
	ResultsArray.SetNum(BufferOffset, EAllowShrinking::No);
}

template<class InT, class OutT>
void FConversionField<InT, OutT>::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	SerializeInternal<InT>(Ar, InputField);
}

template<class InT, class OutT>
bool FConversionField<InT, OutT>::operator==(const FFieldNodeBase& Node)
{
	if (Node.SerializationType() == SerializationType())
	{
		const FConversionField<InT, OutT>* Other = static_cast<const FConversionField<InT, OutT>*>(&Node);
		return Super::operator==(Node)
			&& FieldsEqual(InputField,Other->InputField);
	}
	return false;
}

template<class InT, class OutT>
void FConversionField<InT, OutT>::FillSetupCount(int32& NumOffsets, int32& NumParams) const
{
	if (InputField.IsValid())
	{
		InputField->FillSetupCount(NumOffsets, NumParams);
	}
	Super::FillSetupCount(NumOffsets, NumParams);
	NumParams += 1;
}

template<class InT, class OutT>
void FConversionField<InT, OutT>::FillSetupDatas(TArray<int32>& NodesOffsets, TArray<float>& NodesParams, const float CommandTime) const
{
	if (InputField.IsValid())
	{
		InputField->FillSetupDatas(NodesOffsets, NodesParams, CommandTime);
	}

	Super::FillSetupDatas(NodesOffsets, NodesParams, CommandTime);
	NodesParams.Add(static_cast<float>(InputField != nullptr));
}

template<class InT, class OutT>
float FConversionField<InT, OutT>::EvalMaxMagnitude() const
{
	float MaxMagnitude = 0.0;
	if (InputField.IsValid())
	{
		MaxMagnitude = InputField->EvalMaxMagnitude();
	}
	return MaxMagnitude;
}

template<class InT, class OutT>
void FConversionField<InT, OutT>::ComputeFieldBounds(FVector& MinBounds, FVector& MaxBounds, FVector& CenterPosition) const
{
	FVector MinBoundsA(-FLT_MAX), MaxBoundsA(FLT_MAX);
	if (InputField.IsValid())
	{
		InputField->ComputeFieldBounds(MinBoundsA, MaxBoundsA, CenterPosition);
	}
	MinBounds = MinBoundsA;
	MaxBounds = MaxBoundsA;
}

template class FConversionField<int32, float>;
template class FConversionField<float, int32>;

/**
*  FCullingField<T>
*/

template<class T>
void FCullingField<T>::Evaluate(FFieldContext& Context, TFieldArrayView<T>& Results) const
{
	int32 NumResults = Results.Num();
	int32 NumSamples = Context.SampleIndices.Num();

	const FFieldNode<float> * CullingField = Culling.Get();
	const FFieldNode<T> * InputField = Input.Get();

	if (CullingField != nullptr)
	{
		if (ensureMsgf(CullingField->Type() == FFieldNode<float>::StaticType(),
			TEXT("Field Node CullingFields Culling input expects a float input array.")))
		{
			const uint32 BufferOffset = Context.ScalarResults.Num();
			Context.ScalarResults.SetNum(BufferOffset + NumResults, EAllowShrinking::No);
			TFieldArrayView<float> CullingBuffer(Context.ScalarResults, BufferOffset, NumResults);

			FMemory::Memzero(&Context.ScalarResults[BufferOffset], sizeof(float) * NumResults);
			CullingField->Evaluate(Context, CullingBuffer);

			const uint32 IndexOffset = Context.IndexResults.Num();
			Context.IndexResults.SetNum(IndexOffset + NumResults, EAllowShrinking::No);
			TFieldArrayView<FFieldContextIndex> IndexBuffer(Context.IndexResults, IndexOffset, NumResults);

			int NewEvaluationSize = 0;
			for (int32 SampleIndex = 0; SampleIndex < NumSamples; SampleIndex++)
			{
				const FFieldContextIndex& Index = Context.SampleIndices[SampleIndex];
				{
					if (Operation == EFieldCullingOperationType::Field_Culling_Outside)
					{
						if (CullingBuffer[Index.Result] != 0)
						{
							IndexBuffer[NewEvaluationSize++] = Context.SampleIndices[SampleIndex];
						}
					}
					else
					{
						if (CullingBuffer[Index.Result] == 0)
						{
							IndexBuffer[NewEvaluationSize++] = Context.SampleIndices[SampleIndex];
						} 
					}
				}
			}
			Context.ScalarResults.SetNum(BufferOffset, EAllowShrinking::No);
			if (NewEvaluationSize)
			{
				Context.IndexResults.SetNum(IndexOffset + NewEvaluationSize, EAllowShrinking::No);

				FFieldSystemMetaDataCulling* CullingData = static_cast<FFieldSystemMetaDataCulling*>(Context.MetaData[FFieldSystemMetaData::EMetaType::ECommandData_Culling]);

				if (CullingData)
				{
					CullingData->bCullingActive = true;

					CullingData->CullingIndices.SetNum(NewEvaluationSize, EAllowShrinking::No);
					FMemory::Memcpy(&CullingData->CullingIndices[0], &Context.IndexResults[IndexOffset], NewEvaluationSize * sizeof(FFieldContextIndex));

					if (InputField)
					{
						TFieldArrayView<FFieldContextIndex> LocalIndices(Context.IndexResults, IndexOffset, NewEvaluationSize);
						FFieldContext LocalContext(LocalIndices, Context.SamplePositions, Context.MetaData, Context.TimeSeconds,
							Context.VectorResults, Context.ScalarResults, Context.IntegerResults, Context.IndexResults, CullingData->CullingIndices);
						InputField->Evaluate(LocalContext, Results);
					}
				}
			}
			Context.IndexResults.SetNum(IndexOffset, EAllowShrinking::No);
		}
	}
	else if( InputField!=nullptr)
	{
		InputField->Evaluate(Context, Results);
	}
}

template<class T>
void FCullingField<T>::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	SerializeInternal<float>(Ar, Culling);
	SerializeInternal<T>(Ar, Input);
	SerializeInternal<EFieldCullingOperationType>(Ar, Operation);
}

template<class T>
bool FCullingField<T>::operator==(const FFieldNodeBase& Node)
{
	if (Node.SerializationType() == SerializationType())
	{
		const FCullingField<T>* Other = static_cast<const FCullingField<T>*>(&Node);
		return Super::operator==(Node)
			&& FieldsEqual(Culling,Other->Culling)
			&& FieldsEqual(Input,Other->Input)
			&& Operation == Other->Operation;
	}
	return false;
}

template<class T>
void FCullingField<T>::FillSetupCount(int32& NumOffsets, int32& NumParams) const
{
	if (Culling.IsValid())
	{
		Culling->FillSetupCount(NumOffsets, NumParams);
	}
	if (Input.IsValid())
	{
		Input->FillSetupCount(NumOffsets, NumParams);
	}
	Super::FillSetupCount(NumOffsets, NumParams);
	NumParams += 3;
}

template<class T>
void FCullingField<T>::FillSetupDatas(TArray<int32>& NodesOffsets, TArray<float>& NodesParams, const float CommandTime) const
{
	if (Culling.IsValid())
	{
		Culling->FillSetupDatas(NodesOffsets, NodesParams, CommandTime);
	}
	if (Input.IsValid())
	{
		Input->FillSetupDatas(NodesOffsets, NodesParams, CommandTime);
	}

	Super::FillSetupDatas(NodesOffsets, NodesParams, CommandTime);
	NodesParams.Add(static_cast<float>(Culling != nullptr));
	NodesParams.Add(static_cast<float>(Input != nullptr));
	NodesParams.Add(static_cast<float>(Operation));
}

template<class T>
float FCullingField<T>::EvalMaxMagnitude() const
{
	float MaxMagnitude = 0.0;
	if (Input.IsValid())
	{
		MaxMagnitude = Input->EvalMaxMagnitude();
	}
	return  MaxMagnitude;
}

template<class T>
void FCullingField<T>::ComputeFieldBounds(FVector& MinBounds, FVector& MaxBounds, FVector& CenterPosition) const
{
	FVector MinBoundsA(-FLT_MAX), MaxBoundsA(FLT_MAX), MinBoundsB(-FLT_MAX), MaxBoundsB(FLT_MAX);
	FVector CenterPositionA = FVector::Zero(), CenterPositionB = FVector::Zero();

	if (Culling.IsValid())
	{
		Culling->ComputeFieldBounds(MinBoundsA, MaxBoundsA, CenterPositionA);
	}
	if (Input.IsValid())
	{
		Input->ComputeFieldBounds(MinBoundsB, MaxBoundsB, CenterPositionB);
	}

	CenterPosition = FVector::Zero();
	int32 NumCenters = 0;
	if (CenterPositionA != FVector::Zero())
	{
		CenterPosition += CenterPositionA;
		++NumCenters;
	}
	if (CenterPositionB != FVector::Zero())
	{
		CenterPosition += CenterPositionB;
		++NumCenters;
	}
	CenterPosition = (NumCenters > 0) ? CenterPosition / NumCenters : FVector::Zero();

	if (Operation == EFieldCullingOperationType::Field_Culling_Inside)
	{
		MinBounds = MinVector(MinBoundsA, MinBoundsB);
		MaxBounds = MaxVector(MaxBoundsA, MaxBoundsB);
	}
	else if (Operation == EFieldCullingOperationType::Field_Culling_Outside)
	{
		MinBounds = MaxVector(MinBoundsA, MinBoundsB);
		MaxBounds = MinVector(MaxBoundsA, MaxBoundsB);
	}
}

template class FCullingField<int32>;
template class FCullingField<float>;
template class FCullingField<FVector>;


/**
* FReturnResultsTerminal<T>
*/
template<class T>
void FReturnResultsTerminal<T>::Evaluate(FFieldContext& Context, TFieldArrayView<T>& Results) const
{
	if (ensureMsgf(Context.MetaData.Contains(FFieldSystemMetaData::EMetaType::ECommandData_Results),
		TEXT("Return results nodes can only be used upstream from a 'results expector', for example as an input "
			"to a Operator Node . See documentation for details.")))
	{
		FFieldSystemMetaDataResults<T> * ResultsMetaData = static_cast<FFieldSystemMetaDataResults<T>*>(Context.MetaData[FFieldSystemMetaData::EMetaType::ECommandData_Results]);
		ensure(ResultsMetaData->Results.Num() == Results.Num());

		int32 NumSamples = Context.SampleIndices.Num();
		for (int32 SampleIndex = 0; SampleIndex < NumSamples; SampleIndex++)
		{
			Results[Context.SampleIndices[SampleIndex].Result] = ResultsMetaData->Results[Context.SampleIndices[SampleIndex].Result];
		}
	}
}
template<class T>
void FReturnResultsTerminal<T>::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
}
template<class T>
bool FReturnResultsTerminal<T>::operator==(const FFieldNodeBase& Node)
{
	if (Node.SerializationType() == SerializationType())
	{
		return true;
	}
	return false;
}
template class FReturnResultsTerminal<int32>;
template class FReturnResultsTerminal<float>;
template class FReturnResultsTerminal<FVector>;
