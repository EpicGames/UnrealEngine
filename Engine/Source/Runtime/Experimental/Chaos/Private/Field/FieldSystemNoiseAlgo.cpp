// Copyright Epic Games, Inc. All Rights Reserved.

#include "Field/FieldSystemNoiseAlgo.h"

namespace Field
{
	// Basic scalar implementation of Perlin's improved noise, copied from VectorVM to remove engine dependency
	// http://mrl.nyu.edu/~perlin/noise/
	namespace PerlinNoise
	{
		int32 P[512] =
		{
			151,160,137,91,90,15,
			131,13,201,95,96,53,194,233,7,225,140,36,103,30,69,142,8,99,37,240,21,10,23,
			190, 6,148,247,120,234,75,0,26,197,62,94,252,219,203,117,35,11,32,57,177,33,
			88,237,149,56,87,174,20,125,136,171,168, 68,175,74,165,71,134,139,48,27,166,
			77,146,158,231,83,111,229,122,60,211,133,230,220,105,92,41,55,46,245,40,244,
			102,143,54, 65,25,63,161, 1,216,80,73,209,76,132,187,208, 89,18,169,200,196,
			135,130,116,188,159,86,164,100,109,198,173,186, 3,64,52,217,226,250,124,123,
			5,202,38,147,118,126,255,82,85,212,207,206,59,227,47,16,58,17,182,189,28,42,
			223,183,170,213,119,248,152, 2,44,154,163, 70,221,153,101,155,167, 43,172,9,
			129,22,39,253, 19,98,108,110,79,113,224,232,178,185, 112,104,218,246,97,228,
			251,34,242,193,238,210,144,12,191,179,162,241, 81,51,145,235,249,14,239,107,
			49,192,214, 31,181,199,106,157,184, 84,204,176,115,121,50,45,127, 4,150,254,
			138,236,205,93,222,114,67,29,24,72,243,141,128,195,78,66,215,61,156,180,
			151,160,137,91,90,15,
			131,13,201,95,96,53,194,233,7,225,140,36,103,30,69,142,8,99,37,240,21,10,23,
			190, 6,148,247,120,234,75,0,26,197,62,94,252,219,203,117,35,11,32,57,177,33,
			88,237,149,56,87,174,20,125,136,171,168, 68,175,74,165,71,134,139,48,27,166,
			77,146,158,231,83,111,229,122,60,211,133,230,220,105,92,41,55,46,245,40,244,
			102,143,54, 65,25,63,161, 1,216,80,73,209,76,132,187,208, 89,18,169,200,196,
			135,130,116,188,159,86,164,100,109,198,173,186, 3,64,52,217,226,250,124,123,
			5,202,38,147,118,126,255,82,85,212,207,206,59,227,47,16,58,17,182,189,28,42,
			223,183,170,213,119,248,152, 2,44,154,163, 70,221,153,101,155,167, 43,172,9,
			129,22,39,253, 19,98,108,110,79,113,224,232,178,185, 112,104,218,246,97,228,
			251,34,242,193,238,210,144,12,191,179,162,241, 81,51,145,235,249,14,239,107,
			49,192,214, 31,181,199,106,157,184, 84,204,176,115,121,50,45,127, 4,150,254,
			138,236,205,93,222,114,67,29,24,72,243,141,128,195,78,66,215,61,156,180
		};

		FORCEINLINE float Lerp(float X, float A, float B)
		{
			return A + X * (B - A);
		}

		FORCEINLINE float Fade(float X)
		{
			return X * X * X * (X * (X * 6 - 15) + 10);
		}

		FORCEINLINE float Grad(int32 hash, float x, float y, float z)
		{
			hash &= 15;
			float u = (hash < 8) ? x : y;
			float v = (hash < 4) ? y : ((hash == 12 || hash == 14) ? x : z);
			return ((hash & 1) == 0 ? u : -u) + ((hash & 2) == 0 ? v : -v);
		}

		void Sample(float* Dst, float X, float Y, float Z)
		{
			float Xfl = FMath::FloorToFloat(X);
			float Yfl = FMath::FloorToFloat(Y);
			float Zfl = FMath::FloorToFloat(Z);
			int32 Xi = (int32)(Xfl) & 255;
			int32 Yi = (int32)(Yfl) & 255;
			int32 Zi = (int32)(Zfl) & 255;
			X -= Xfl;
			Y -= Yfl;
			Z -= Zfl;
			float Xm1 = X - 1.0f;
			float Ym1 = Y - 1.0f;
			float Zm1 = Z - 1.0f;

			int32 A = P[Xi] + Yi;
			int32 AA = P[A] + Zi;	int32 AB = P[A + 1] + Zi;

			int32 B = P[Xi + 1] + Yi;
			int32 BA = P[B] + Zi;	int32 BB = P[B + 1] + Zi;

			float U = Fade(X);
			float V = Fade(Y);
			float W = Fade(Z);

			*Dst =
				Lerp(W,
					Lerp(V,
						Lerp(U,
							Grad(P[AA], X, Y, Z),
							Grad(P[BA], Xm1, Y, Z)),
						Lerp(U,
							Grad(P[AB], X, Ym1, Z),
							Grad(P[BB], Xm1, Ym1, Z))),
					Lerp(V,
						Lerp(U,
							Grad(P[AA + 1], X, Y, Zm1),
							Grad(P[BA + 1], Xm1, Y, Zm1)),
						Lerp(U,
							Grad(P[AB + 1], X, Ym1, Zm1),
							Grad(P[BB + 1], Xm1, Ym1, Zm1))));
		}
	}
}