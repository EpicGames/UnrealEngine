// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/DebugSolverTasks.h"

#if CHAOS_DEBUG_SUBSTEP
#include "PhysicsSolver.h"
#include "ChaosLog.h"

namespace Chaos
{
	FDebugSolverTask::FDebugSolverTask(TFunction<void()> InStepFunction, FDebugSubstep& InDebugSubstep)
		: StepFunction(InStepFunction)
		, DebugSubstep(InDebugSubstep)
	{
	}

	/** Solver advances. */
	void FDebugSolverTask::DoWork()
	{
		// Save thread id to make sure there is no substep added inside any parallel for or some other thread forks
		DebugSubstep.AssumeThisThread();

		// Add initial step to pause on
		DebugSubstep.Add(true, TEXT("Debug thread start"));

		// Solver advance loop
		while (DebugSubstep.IsEnabled())
		{
			// Solver advance
			StepFunction();
			// Add end-of-solver-advance step
			DebugSubstep.Add(true, TEXT("Debug thread step"));
		}

		// Add one final step
		DebugSubstep.Add(true, TEXT("Debug thread exit"));
	}

	void FDebugSolverTasks::DebugStep(FPhysicsSolverBase* Solver, TFunction<void()> StepFunction)
	{
		// Retrieve debug thread pointer for this solver.
		// It must have been already added unless the single thread mode has just been switched on.
		FAsyncTask<FDebugSolverTask>** DebugSolverTask = SolverToTaskMap.Find(Solver);
		if (!DebugSolverTask)
		{
			DebugSolverTask = &SolverToTaskMap.Add(Solver, nullptr);
		}

#if TODO_REIMPLEMENT_DEBUG_SUBSTEP
		// Sync/advance debug substep system for this solver
		FDebugSubstep& DebugSubstep = Solver->GetDebugSubstep();
		const bool bNeedsDebugThreadRunning = DebugSubstep.SyncAdvance(Solver->Enabled());

		// Update debug thread status
		const bool bIsDebugThreadRunning = !!*DebugSolverTask;
		if (bIsDebugThreadRunning != bNeedsDebugThreadRunning)
		{
			if (bNeedsDebugThreadRunning)
			{
				UE_LOGF(LogChaosThread, Verbose, "[Physics Thread] Spawning new debug thread");
				// Spawn a new debug thread running the solver step
				*DebugSolverTask = new FAsyncTask<FDebugSolverTask>(StepFunction, DebugSubstep);
				(*DebugSolverTask)->StartBackgroundTask();
			}
			else
			{
				// Delete AsyncTask
				UE_LOGF(LogChaosThread, Verbose, "[Physics Thread] Removing debug thread");
				(*DebugSolverTask)->EnsureCompletion(false);
				delete *DebugSolverTask;
				*DebugSolverTask = nullptr;
				UE_LOGF(LogChaosThread, Verbose, "[Physics Thread] Completed and deleted debug thread");
			}
		}

		// Step solver in normal fashion
		if (!bNeedsDebugThreadRunning)
		{
			StepFunction();
		}
#endif
	}

	void FDebugSolverTasks::Add(FPhysicsSolverBase* Solver)
	{
		// Add solver to task map
		SolverToTaskMap.Add(Solver, nullptr);

#if TODO_REIMPLEMENT_DEBUG_SUBSTEP
		// Reinit debug substep
		UE_LOGF(LogChaosThread, Verbose, "[Physics Thread] Initializing debug thread");
		Solver->GetDebugSubstep().Initialize();
#endif
	}

	void FDebugSolverTasks::Remove(FPhysicsSolverBase* Solver)
	{
#if TODO_REIMPLEMENT_DEBUG_SUBSTEP
		// Remove the debug advance task for this solver, if any was created
		FAsyncTask<FDebugSolverTask>* DebugSolverTask;
		const bool bFound = SolverToTaskMap.RemoveAndCopyValue(Solver, DebugSolverTask);
		if (bFound && DebugSolverTask)
		{
			// Unpause debug thread and exit
			UE_LOGF(LogChaosThread, Verbose, "[Physics Thread] Shutting down debug thread");
			Solver->GetDebugSubstep().Release();  // A call to Release() here means it won't debug substep until another call to Initialize() is made.

			// Wait for thread completion and delete the task
			UE_LOGF(LogChaosThread, Verbose, "[Physics Thread] Deleting debug async task");
			DebugSolverTask->EnsureCompletion(false);
			delete DebugSolverTask;
		}
#endif
	}

	void FDebugSolverTasks::Shutdown()
	{
#if TODO_REIMPLEMENT_DEBUG_SUBSTEP
		// Iterate through all solver tasks in the map
		for (auto& SolverTask: SolverToTaskMap)
		{
			// Shutdown debug task if needed
			UE_LOGF(LogChaosThread, Verbose, "[Physics Thread] Shutting debug thread");
			SolverTask.Key->GetDebugSubstep().Shutdown();  // A call to Shutdown() here means it will be able to debug substep again , even without another call to Initialize() made (eg. switch back and forth between threading modes).

			// Wait for thread completion and delete the task
			if (SolverTask.Value)
			{
				UE_LOGF(LogChaosThread, Verbose, "[Physics Thread] Deleting debug async task");
				SolverTask.Value->EnsureCompletion(false);
				delete SolverTask.Value;
				SolverTask.Value = nullptr;
			}
		}
#endif
	}

}  // namespace Chaos

#endif  // #if CHAOS_DEBUG_SUBSTEP
