// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/Framework/PhysicsProxyBase.h"
#include "Chaos/Framework/PhysicsSolverBase.h"

namespace RenderInterpolationCVars
{
	float RenderInterpErrorCorrectionDuration = 0.3f;
	static FAutoConsoleVariableRef CVarRenderInterpErrorCorrectionDuration(TEXT("p.RenderInterp.ErrorCorrectionDuration"), RenderInterpErrorCorrectionDuration, TEXT("How long in seconds to apply error correction over."));

	float RenderInterpMaximumErrorCorrectionBeforeSnapping = 250.0f;
	static FAutoConsoleVariableRef CVarRenderInterpErrorCorrectionMaximumError(TEXT("p.RenderInterp.MaximumErrorCorrectionBeforeSnapping"), RenderInterpMaximumErrorCorrectionBeforeSnapping, TEXT("Maximum error correction in cm before we stop interpolating and snap to target. NOTE: MaximumErrorCorrectionDesyncTimeBeforeSnapping can set a larger distance if enabled."));

	float RenderInterpMaximumErrorCorrectionDesyncTimeBeforeSnapping = 0.6f;
	static FAutoConsoleVariableRef CVarRenderInterpErrorCorrectionDesyncTime(TEXT("p.RenderInterp.MaximumErrorCorrectionDesyncTimeBeforeSnapping"), RenderInterpMaximumErrorCorrectionDesyncTimeBeforeSnapping, TEXT("Time multiplied by the particles velocity to get the distance that error correction will be performed within without snapping, disable by setting a negative value. NOTE: MaximumErrorCorrectionBeforeSnapping will act as a lowest distance clamp."));

	float RenderInterpErrorVelocitySmoothingDuration = 0.3f;
	static FAutoConsoleVariableRef CVarRenderInterpErrorVelocitySmoothingDuration(TEXT("p.RenderInterp.ErrorVelocitySmoothingDuration"), RenderInterpErrorVelocitySmoothingDuration, TEXT("How long in seconds to apply error velocity smoothing correction over, should be smaller than or equal to p.RenderInterp.ErrorCorrectionDuration. RENDERINTERPOLATION_VELOCITYSMOOTHING needs to be defined."));

	float RenderInterpErrorDirectionalDecayMultiplier = 0.0f;
	static FAutoConsoleVariableRef CVarRenderInterpErrorDirectionalDecayMultiplier(TEXT("p.RenderInterp.DirectionalDecayMultiplier"), RenderInterpErrorDirectionalDecayMultiplier, TEXT("Decay error offset in the direction that the physics object is moving, value is multiplier of projected offset direction, 0.25 means a 25% decay of the magnitude in the direction of physics travel. Deactivate by setting to 0."));

	bool bRenderInterpErrorVelocityCorrection = false;
	static FAutoConsoleVariableRef CVarRenderInterpErrorVelocityCorrection(TEXT("p.RenderInterp.ErrorVelocityCorrection"), bRenderInterpErrorVelocityCorrection, TEXT("EXPERIMENTAL - Take incoming velocity into consideration when performing render interpolation, the correction will be more organic but might result in clipping and it's heavier for memory and CPU."));

	bool bRenderInterpDebugDraw = false;
	static FAutoConsoleVariableRef CVarRenderInterpDebugDraw(TEXT("p.RenderInterp.DebugDraw"), bRenderInterpDebugDraw, TEXT("Draw debug lines for physics render interpolation, also needs p.Chaos.DebugDraw.Enabled set"));

	bool bRenderInterpDebugDrawGC = false;
	static FAutoConsoleVariableRef CVarRenderInterpDebugDrawGC(TEXT("p.RenderInterp.DebugDraw.GC"), bRenderInterpDebugDrawGC, TEXT("Enable render interpolation debug draw for Geometry Collections"));

	float RenderInterpDebugDrawZOffset = 0.0f;
	static FAutoConsoleVariableRef CVarRenderInterpDebugDrawZOffset(TEXT("p.RenderInterp.DebugDrawZOffset"), RenderInterpDebugDrawZOffset, TEXT("Add Z axis offset to DebugDraw calls for Render Interpolation."));

	bool bRenderInterpApplyExponentialDecay = false;
	static FAutoConsoleVariableRef CVarRenderInterpApplyExponentialDecay(TEXT("p.RenderInterp.ApplyExponentialDecay"), bRenderInterpApplyExponentialDecay, TEXT("When enabled a post-resim error will decay exponentially (instead of linearly) based on half-life time set in ExponentialDecayLinearHalfLife and ExponentialDecayAngularHalfLife."));

	float RenderInterpExponentialDecayLinearHalfLife = 0.06f;
	static FAutoConsoleVariableRef CVarRenderInterpExponentialDecayLinearHalfLife(TEXT("p.RenderInterp.ExponentialDecayLinearHalfLife"), RenderInterpExponentialDecayLinearHalfLife, TEXT("Sets the positional half-life time for when bApplyExponentialDecay is enabled."));

	float RenderInterpExponentialDecayAngularHalfLife = 0.06f;
	static FAutoConsoleVariableRef CVarRenderInterpExponentialDecayAngularHalfLife(TEXT("p.RenderInterp.ExponentialDecayAngularHalfLife"), RenderInterpExponentialDecayAngularHalfLife, TEXT("Sets the rotational half-life time for when bApplyExponentialDecay is enabled."));

	float RenderInterpMinimumLinearThreshold = 0.1f;
	static FAutoConsoleVariableRef CVarRenderInterpMinimumLinearThreshold(TEXT("p.RenderInterp.MinimumLinearThreshold"), RenderInterpMinimumLinearThreshold, TEXT("Squared value, when the remaining render error is below this we clear it, if ApplyExponentialDecay is enabled."));

	float RenderInterpMinimumAngularThreshold = 0.001f;
	static FAutoConsoleVariableRef CVarRenderInterpMinimumAngularThreshold(TEXT("p.RenderInterp.MinimumAngularThreshold"), RenderInterpMinimumAngularThreshold, TEXT("When the remaining render error angle is below this we clear it, if ApplyExponentialDecay is enabled."));

	bool bSnapToLatest = false;
	static FAutoConsoleVariableRef CVarRenderInterpSnapToLatest(TEXT("p.RenderInterp.SnapToLatest"), bSnapToLatest, TEXT("Always snap to the latest physics state, instead of interpolating."));
}

IPhysicsProxyBase::~IPhysicsProxyBase()
{
	if (GetSolver<Chaos::FPhysicsSolverBase>())
	{
		GetSolver<Chaos::FPhysicsSolverBase>()->RemoveDirtyProxyFromHistory_Internal(this);
		GetSolver<Chaos::FPhysicsSolverBase>()->RemoveDirtyProxy(this);
	}
}

int32 IPhysicsProxyBase::GetSolverSyncTimestamp_External() const
{
	if (Chaos::FPhysicsSolverBase* SolverBase = GetSolverBase())
	{
		return SolverBase->GetMarshallingManager().GetExternalTimestamp_External();
	}

	return INDEX_NONE;
}

const bool FProxyInterpolationError::IsErrorSmoothing() const
{
	const bool ApplyExponentialDecay = ErrorInterpolationSettings.Get() ? ErrorInterpolationSettings.Get()->bApplyExponentialDecay : RenderInterpolationCVars::bRenderInterpApplyExponentialDecay;
	if (ApplyExponentialDecay)
	{
		const float MinimumLinearThreshold = ErrorInterpolationSettings.Get() ? ErrorInterpolationSettings.Get()->MinimumLinearThreshold : RenderInterpolationCVars::RenderInterpMinimumLinearThreshold;
		const float MinimumAngularThreshold = ErrorInterpolationSettings.Get() ? ErrorInterpolationSettings.Get()->MinimumAngularThreshold : RenderInterpolationCVars::RenderInterpMinimumAngularThreshold;
		
		return ErrorX.SizeSquared() > MinimumLinearThreshold
			|| ErrorR.GetAngle() > MinimumAngularThreshold;
	}
	return LastSimTick < EndDecayTick;
}


void FProxyInterpolationError::AccumulateErrorXR(const Chaos::FVec3 X, const FQuat R, const int32 CurrentSimTick, const int32 ErrorSmoothDuration)
{
	LastSimTick = CurrentSimTick - 1; // Error is from the previous simulation tick, not the current
	EndDecayTick = LastSimTick + ErrorSmoothDuration;
	SimTicks = 0;

	const bool ApplyExponentialDecay = ErrorInterpolationSettings.Get() ? ErrorInterpolationSettings.Get()->bApplyExponentialDecay : RenderInterpolationCVars::bRenderInterpApplyExponentialDecay;
	if (ApplyExponentialDecay || IsErrorSmoothing())
	{
		ErrorX += X;
		ErrorR = R * ErrorR;
	}
	else
	{
		Reset();
	}
}

const bool FProxyInterpolationError::UpdateError(const int32 CurrentSimTick, const Chaos::FReal AsyncFixedTimeStep)
{
	// Cache how many simulation ticks have passed since last call
	SimTicks = CurrentSimTick - LastSimTick;
	LastSimTick = CurrentSimTick;

	if (SimTicks > 0)
	{
		return DecayError(AsyncFixedTimeStep);
	}
	return false;
}

const bool FProxyInterpolationError::DirectionalDecay(Chaos::FVec3 Direction, float ErrorDirectionalDecayMultiplier)
{
	if (ErrorX.IsNearlyZero())
	{
		return false;
	}

	ErrorDirectionalDecayMultiplier = ErrorInterpolationSettings.Get() ? ErrorInterpolationSettings.Get()->ErrorDirectionalDecayMultiplier : ErrorDirectionalDecayMultiplier;
	if (ErrorDirectionalDecayMultiplier > 0.0f && IsErrorSmoothing() && SimTicks > 0)
	{
		Chaos::FVec3 DirProjection = Direction.ProjectOnTo(ErrorX) * ErrorDirectionalDecayMultiplier;
		const Chaos::FRealDouble DotProd = Chaos::FVec3::DotProduct(DirProjection, ErrorX);

		if (DotProd > 0.0f)
		{
			if (DirProjection.SizeSquared() < ErrorX.SizeSquared())
			{
				ErrorX -= DirProjection;
			}
			else
			{
				ErrorX = Chaos::FVec3::Zero();
			}
			return true;
		}
	}
	return false;
}

const bool FProxyInterpolationError::DecayError(const Chaos::FReal AsyncFixedTimeStep)
{
	if (SimTicks <= 0)
	{
		return false;
	}

	if (!IsErrorSmoothing())
	{
		Reset();
		return false;
	}

	const bool ApplyExponentialDecay = ErrorInterpolationSettings.Get() ? ErrorInterpolationSettings.Get()->bApplyExponentialDecay : RenderInterpolationCVars::bRenderInterpApplyExponentialDecay;

	if (ApplyExponentialDecay)
	{
		ErrorXPrev = ErrorX;
		ErrorRPrev = ErrorR;

		const float ExponentialDecayLinearHalfLife = ErrorInterpolationSettings.Get() ? ErrorInterpolationSettings.Get()->ExponentialDecayLinearHalfLife : RenderInterpolationCVars::RenderInterpExponentialDecayLinearHalfLife;
		const float ExponentialDecayAngularHalfLife = ErrorInterpolationSettings.Get() ? ErrorInterpolationSettings.Get()->ExponentialDecayAngularHalfLife : RenderInterpolationCVars::RenderInterpExponentialDecayAngularHalfLife;
		const double SimulatedTime = SimTicks * AsyncFixedTimeStep;

		if (ExponentialDecayLinearHalfLife > UE_SMALL_NUMBER)
		{
			const double DecayAmountX = FMath::Exp2(-SimulatedTime / ExponentialDecayLinearHalfLife); // Same as FMath::Pow(0.5f, SimulatedTime / RenderInterpolationCVars::RenderInterpExponentialDecayLinearHalfLife)
			ErrorX = ErrorX * DecayAmountX;
		}
		else
		{
			ErrorX = Chaos::FVec3::ZeroVector;
		}

		if (ExponentialDecayAngularHalfLife > UE_SMALL_NUMBER)
		{
			const double DecayAmountR = FMath::Exp2(-SimulatedTime / ExponentialDecayAngularHalfLife); // Same as FMath::Pow(0.5f, SimulatedTime / RenderInterpolationCVars::RenderInterpExponentialDecayAngularHalfLife)
			ErrorR = FQuat::Slerp(FQuat::Identity, ErrorR, DecayAmountR);
		}
		else
		{
			ErrorR = FQuat::Identity;
		}

		return true;
	}
	else
	{
		// Linear decay
		// Example: If we want to decay an error of 100 over 10ticks (i.e. 10% each tick)
		// First step:  9/10 = 0.9   |  100 * 0.9  = 90 error
		// Second step: 8/9  = 0.888 |  90 * 0.888 = 80 error
		// Third step: 7/8  = 0.875 |  80 * 0.875 = 70 error
		// etc.
		const int32 ErrorSmoothingCount = EndDecayTick - LastSimTick;
		int32 Ticks = FMath::Clamp(SimTicks, 0, ErrorSmoothingCount);
		for (; Ticks > 0; Ticks--)
		{
			Chaos::FRealSingle Alpha = Chaos::FRealSingle(ErrorSmoothingCount - 1) / Chaos::FRealSingle(ErrorSmoothingCount);
			Alpha = FMath::Clamp(Alpha, 0.f, 1.f);

			ErrorXPrev = ErrorX;
			ErrorX *= Alpha;
			ErrorRPrev = ErrorR;
			ErrorR = FQuat::Slerp(FQuat::Identity, ErrorR, Alpha);
		}
		return true;
	}
}