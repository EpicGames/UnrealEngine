// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Threading.h"
#include "ChaosCheck.h"

#if PHYSICS_THREAD_CONTEXT
UE_DEFINE_THREAD_SINGLETON_TLS(Chaos::FPhysicsThreadContext, CHAOS_API)
#endif

namespace Chaos
{
	// uint32 pair for tracking reads and writes per thread.
	struct FLockDepthData
	{
		uint32 ReadDepth = 0;
		uint32 WriteDepth = 0;

		bool HasLocks() const 
		{
			return (ReadDepth + WriteDepth) > 0;
		}
	};
	
	struct FMultiInstanceLockDepthData
	{
		FLockDepthData& Get(void* InstancePtr)
		{
			return InstanceDepths.FindOrAdd(InstancePtr);
		}

		void IncRead(void* InstancePtr)
		{
			++(Get(InstancePtr).ReadDepth);
		}

		void IncWrite(void* InstancePtr)
		{
			++(Get(InstancePtr).WriteDepth);
		}

		void DecRead(void* InstancePtr)
		{
			FLockDepthData& Data = Get(InstancePtr);
			--Data.ReadDepth;

			if(!Data.HasLocks())
			{
				InstanceDepths.Remove(InstancePtr);
			}
		}

		void DecWrite(void* InstancePtr)
		{
			FLockDepthData& Data = Get(InstancePtr);
			--Data.WriteDepth;

			if(!Data.HasLocks())
			{
				InstanceDepths.Remove(InstancePtr);
			}
		}

	private:
		TMap<void*, FLockDepthData> InstanceDepths;
	};
	thread_local static FMultiInstanceLockDepthData GThreadLockCheckData;

	namespace ThreadingPrivate
	{
#if CHAOS_SCENE_LOCK_CHECKS
		void CheckLockReadAssumption(const TCHAR* Context)
		{
			// Holding either a read lock or a write lock enables reading, so if both are zero we emit an error
			if(GThreadLockCheckData.ReadDepth == 0 && GThreadLockCheckData.WriteDepth == 0)
			{
				UE_LOGF(LogChaos, Error, "Thread %u called %ls without taking a read lock. Wrap the function call in a scene read lock to safely read data.", FPlatformTLS::GetCurrentThreadId(), Context);
				CHAOS_ENSURE(false);
			}
		}

		void CheckLockWriteAssumption(const TCHAR* Context)
		{
			// Must have at least one write scope to pass the assumption
			if(GThreadLockCheckData.WriteDepth == 0)
			{
				UE_LOGF(LogChaos, Error, "Thread %u called %ls without taking a write lock. Wrap the function call in a scene write lock to safely write data.", FPlatformTLS::GetCurrentThreadId(), Context);
				CHAOS_ENSURE(false);
			}
		}
#endif

		void IncReadDepth(void* Instance)
		{
			GThreadLockCheckData.IncRead(Instance);
		}

		void IncWriteDepth(void* Instance)
		{
			GThreadLockCheckData.IncWrite(Instance);
		}

		void DecReadDepth(void* Instance)
		{
			GThreadLockCheckData.DecRead(Instance);
		}

		void DecWriteDepth(void* Instance)
		{
			GThreadLockCheckData.DecWrite(Instance);
		}

		uint32 GetThreadReadDepth(void* Instance)
		{
			return GThreadLockCheckData.Get(Instance).ReadDepth;
		}

	}

#if UE_AUTORTFM && UE_WITH_REMOTE_OBJECT_HANDLE
	FPhysSceneLockCallbacks GPhysSceneLockRemoteObjectCallbacks;
#endif
}
