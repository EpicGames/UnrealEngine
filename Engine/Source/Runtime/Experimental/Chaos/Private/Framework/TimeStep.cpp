// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/TimeStep.h"

#include "ChaosLog.h"
#include "HAL/PlatformTime.h"
#include "HAL/PlatformProcess.h"
#include "HAL/IConsoleManager.h"

TAutoConsoleVariable<float> CVarVariableTickCap(
	TEXT("p.Chaos.Timestep.VariableCapped.Cap"),
	0.0667f,
	TEXT("Time in seconds to set as the cap when using a ranged timestep for Chaos.")
);

namespace Chaos
{
	void FFixedTimeStep::Reset()
	{
		CurrentTime = 0.0f;
		LastTime = FPlatformTime::Seconds();
		TargetDt = 1.0f / 60.0f;
		ActualDt = TargetDt;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	void FFixedTimeStep::Update()
	{
		CurrentTime = FPlatformTime::Seconds();
		ActualDt = static_cast<float>(CurrentTime - LastTime);

		if(ActualDt > TargetDt)
		{
			UE_LOGF(LogChaosDebug, Verbose, "PhysAdvance: Exceeded requested Dt of %.3f (%.2fFPS). Ran for %.3f", TargetDt, 1.0f / TargetDt, ActualDt);
		}
		else
		{
			UE_LOGF(LogChaosDebug, Verbose, "PhysAdvance: Advance took %.3f, sleeping for %.3f to reach target Dt of %.3f (%.2fFPS)", ActualDt, TargetDt - ActualDt, TargetDt, 1.0f / TargetDt);

			// #BG TODO need some way to handle abandonning this when the gamethread requests a sync
			// Or just running more commands in general otherwise this is dead time.
			FPlatformProcess::Sleep(TargetDt - ActualDt);
		}

		LastTime = FPlatformTime::Seconds();
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	float FFixedTimeStep::GetCalculatedDt() const
	{
		// Don't pass anything variable here - just the target as we always run fixed
		return TargetDt;
	}

	float FFixedTimeStep::GetActualDt() const
	{
		return ActualDt;
	}

	void FFixedTimeStep::SetTarget(float InTarget)
	{
		TargetDt = InTarget;
	}

	float FFixedTimeStep::GetTarget() const
	{
		return TargetDt;
	}

	void FVariableTimeStep::Reset()
	{
		LastTime = FPlatformTime::Seconds();
		Dt = 1.0f / 60.0f;
	}

	void FVariableTimeStep::Update()
	{
		double CurrentTime = FPlatformTime::Seconds();
		Dt = static_cast<float>(FPlatformTime::Seconds() - LastTime);
		LastTime = CurrentTime;
	}

	float FVariableTimeStep::GetCalculatedDt() const
	{
		return Dt;
	}

	void FVariableMinimumWithCapTimestep::Reset()
	{
		CurrentTime = 0.0f;
		LastTime = FPlatformTime::Seconds();
		TargetDt = 1.0f / 60.0f;
		Dt = TargetDt;
		ActualDt = TargetDt;
	}

	void FVariableMinimumWithCapTimestep::Update()
	{
		CurrentTime = FPlatformTime::Seconds();
		ActualDt = static_cast<float>(CurrentTime - LastTime);

		if(ActualDt < TargetDt)
		{
			FPlatformProcess::Sleep(TargetDt - ActualDt);
			Dt = TargetDt;
		}
		else
		{
			float Cap = CVarVariableTickCap.GetValueOnAnyThread();
			Dt = FMath::Min(ActualDt, Cap);
		}

		LastTime = FPlatformTime::Seconds();
	}

	float FVariableMinimumWithCapTimestep::GetCalculatedDt() const
	{
		return Dt;
	}

	float FVariableMinimumWithCapTimestep::GetActualDt() const
	{
		return ActualDt;
	}

	void FVariableMinimumWithCapTimestep::SetTarget(float InTarget)
	{
		TargetDt = InTarget;
	}

	void FVariableWithCapTimestep::Reset()
	{
		CurrentTime = 0.0f;
		LastTime = FPlatformTime::Seconds();
		Dt = 1.0f / 60.0f;
		ActualDt = Dt;
	}

	void FVariableWithCapTimestep::Update()
	{
		CurrentTime = FPlatformTime::Seconds();
		ActualDt = static_cast<float>(CurrentTime - LastTime);

		float Cap = CVarVariableTickCap.GetValueOnAnyThread();
		Dt = FMath::Min(ActualDt, Cap);

		LastTime = FPlatformTime::Seconds();
	}

	float FVariableWithCapTimestep::GetCalculatedDt() const
	{
		return Dt;
	}

	float FVariableWithCapTimestep::GetActualDt() const
	{
		return ActualDt;
	}

}
