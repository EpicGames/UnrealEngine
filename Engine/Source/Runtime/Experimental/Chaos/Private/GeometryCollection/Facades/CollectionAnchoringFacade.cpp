// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/Facades/CollectionAnchoringFacade.h"
#include "GeometryCollection/TransformCollection.h"

namespace Chaos::Facades
{
	FCollectionAnchoringFacade::FCollectionAnchoringFacade(FManagedArrayCollection& InCollection)
		: InitialDynamicStateAttribute(InCollection, "InitialDynamicState", FTransformCollection::TransformGroup)
	    , AnchoredAttribute(InCollection, "Anchored", FTransformCollection::TransformGroup)
	{}

	FCollectionAnchoringFacade::FCollectionAnchoringFacade(const FManagedArrayCollection& InCollection)
		: InitialDynamicStateAttribute(InCollection, "InitialDynamicState", FTransformCollection::TransformGroup)
		, AnchoredAttribute(InCollection, "Anchored", FTransformCollection::TransformGroup)
	{}

	bool FCollectionAnchoringFacade::IsValid() const
	{
		return InitialDynamicStateAttribute.IsValid() && AnchoredAttribute.IsValid();
	}

	bool FCollectionAnchoringFacade::HasInitialDynamicStateAttribute() const
	{
		return InitialDynamicStateAttribute.IsValid();
	}

	bool FCollectionAnchoringFacade::HasAnchoredAttribute() const
	{
		return AnchoredAttribute.IsValid();
	}

	void FCollectionAnchoringFacade::AddAnchoredAttribute()
	{
		check(!IsConst());
		return AnchoredAttribute.AddAndFill(false);
	}

	void FCollectionAnchoringFacade::CopyAnchoredAttribute(const FCollectionAnchoringFacade& From)
	{
		check(!IsConst());
		if (From.HasAnchoredAttribute())
		{
			AnchoredAttribute.Copy(From.AnchoredAttribute);
		}
	}
	
	EObjectStateType FCollectionAnchoringFacade::GetInitialDynamicState(int32 TransformIndex) const
	{
		const int32 State = InitialDynamicStateAttribute.Get()[TransformIndex];
		return static_cast<EObjectStateType>(State);
	}
	
	void FCollectionAnchoringFacade::SetInitialDynamicState(int32 TransformIndex, EObjectStateType State)
	{
		check(!IsConst());
		// we don't really support sleeping as a dynamic state
		if (ensure(State != EObjectStateType::Sleeping && State < EObjectStateType::Count))
		{
			InitialDynamicStateAttribute.Modify()[TransformIndex] = static_cast<int32>(State);
		}
	}

	void FCollectionAnchoringFacade::SetInitialDynamicState(const TArray<int32>& TransformIndices, EObjectStateType State)
	{
		check(!IsConst());
		// we don't really support sleeping as a dynamic state
		if (ensure(State != EObjectStateType::Sleeping && State < EObjectStateType::Count))
		{
			TManagedArray<int32>& InitialDynamicState = InitialDynamicStateAttribute.Modify();
			for (const int32 TransformIndex: TransformIndices)
			{
				if (InitialDynamicState.IsValidIndex(TransformIndex))
				{
					InitialDynamicState[TransformIndex] = static_cast<int32>(State);
				}
			}
		}
	}
		
	bool FCollectionAnchoringFacade::IsAnchored(int32 TransformIndex) const
	{
		return AnchoredAttribute.IsValid()? AnchoredAttribute.Get()[TransformIndex]: false;
	}

	void FCollectionAnchoringFacade::SetAnchored(int32 TransformIndex, bool bValue)
	{
		check(!IsConst());
		AnchoredAttribute.Modify()[TransformIndex] = bValue;
	}

	void FCollectionAnchoringFacade::SetAnchored(const TArray<int32>& TransformIndices, bool bValue)
	{
		check(!IsConst());
		TManagedArray<bool>& Anchored = AnchoredAttribute.Modify();
		for (const int32 TransformIndex: TransformIndices)
		{
			Anchored[TransformIndex] = bValue;
		}
	}
}