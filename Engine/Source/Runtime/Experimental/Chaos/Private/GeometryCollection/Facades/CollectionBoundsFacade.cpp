// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GeometryCollection.cpp: FGeometryCollection methods.
=============================================================================*/

#include "GeometryCollection/Facades/CollectionBoundsFacade.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/Facades/CollectionHierarchyFacade.h"
#include "GeometryCollection/Facades/CollectionTransformFacade.h"

namespace GeometryCollection::Facades
{

	FBoundsFacade::FBoundsFacade(FManagedArrayCollection& InCollection)
		: ConstCollection(InCollection)
		, Collection(&InCollection)
		, BoundingBoxAttribute(InCollection,"BoundingBox", FGeometryCollection::GeometryGroup)
		, VertexAttribute(InCollection,"Vertex", FGeometryCollection::VerticesGroup, FGeometryCollection::VerticesGroup)
		, BoneMapAttribute(InCollection,"BoneMap", FGeometryCollection::VerticesGroup)
		, TransformToGeometryIndexAttribute(InCollection,"TransformToGeometryIndex", FTransformCollection::TransformGroup)
		, VertexStartAttribute(InCollection, "VertexStart", FGeometryCollection::GeometryGroup)
		, VertexCountAttribute(InCollection, "VertexCount", FGeometryCollection::GeometryGroup)
	{}

	FBoundsFacade::FBoundsFacade(const FManagedArrayCollection& InCollection)
		: ConstCollection(InCollection)
		, Collection(nullptr)
		, BoundingBoxAttribute(InCollection, "BoundingBox", FGeometryCollection::GeometryGroup)
		, VertexAttribute(InCollection, "Vertex", FGeometryCollection::VerticesGroup, FGeometryCollection::VerticesGroup)
		, BoneMapAttribute(InCollection, "BoneMap", FGeometryCollection::VerticesGroup)
		, TransformToGeometryIndexAttribute(InCollection, "TransformToGeometryIndex", FTransformCollection::TransformGroup)
		, VertexStartAttribute(InCollection, "VertexStart", FGeometryCollection::GeometryGroup)
		, VertexCountAttribute(InCollection, "VertexCount", FGeometryCollection::GeometryGroup)
	{}

	//
	//  Initialization
	//

	void FBoundsFacade::DefineSchema()
	{
		check(!IsConst());
		BoundingBoxAttribute.Add();
	}

	bool FBoundsFacade::IsValid() const
	{
		return BoundingBoxAttribute.IsValid();
	}


	void FBoundsFacade::UpdateBoundingBox()
	{
		if (BoundingBoxAttribute.IsValid())
		{
			bool bTranformBased = VertexAttribute.IsValid() && TransformToGeometryIndexAttribute.Num() && BoneMapAttribute.IsValid();

			bool bVertexBased = VertexAttribute.IsValid() && VertexStartAttribute.Num() && VertexCountAttribute.IsValid();

			if (bTranformBased)
			{
				UpdateTransformBasedBoundingBox();
			}
			else if (bVertexBased)
			{
				UpdateVertexBasedBoundingBox();
			}
			else
			{
				TManagedArray<FBox>& BoundingBox = BoundingBoxAttribute.Modify();
				for (int32 bdx = 0; bdx < BoundingBox.Num(); bdx++)
				{
					BoundingBox[bdx].Init();
				}
			}
		}
	}

	void FBoundsFacade::UpdateTransformBasedBoundingBox()
	{
		TManagedArray<FBox>& BoundingBox = BoundingBoxAttribute.Modify();
		const TManagedArray<FVector3f>& Vertex = VertexAttribute.Get();
		const TManagedArray<int32>& BoneMap = BoneMapAttribute.Get();
		const TManagedArray<int32>& TransformToGeometryIndex = TransformToGeometryIndexAttribute.Get();

		for (int32 bdx = 0; bdx < BoundingBox.Num(); bdx++)
		{
			BoundingBox[bdx].Init();
		}

		// Use the mapping stored from the vertices to the transforms to generate a bounding box
		// relative to transform origin. 

		if (BoundingBox.Num())
		{
			// Compute BoundingBox
			for (int32 Idx = 0; Idx < Vertex.Num(); ++Idx)
			{
				int32 TransformIndexValue = BoneMap[Idx];
				BoundingBox[TransformToGeometryIndex[TransformIndexValue]] += FVector(Vertex[Idx]);
			}
		}
	}

	void FBoundsFacade::UpdateVertexBasedBoundingBox()
	{
		TManagedArray<FBox>& BoundingBox = BoundingBoxAttribute.Modify();
		const TManagedArray<FVector3f>& Vertex = VertexAttribute.Get();
		const TManagedArray<int32>& VertexStart = VertexStartAttribute.Get();
		const TManagedArray<int32>& VertexCount = VertexCountAttribute.Get();

		for (int32 bdx = 0; bdx < BoundingBox.Num(); bdx++)
		{
			BoundingBox[bdx].Init();
		}

		// Use the mapping stored from the geometry to the vertices to generate a bounding box.
		// This configuration might not have an assiocated transform.

		for (int32 Gdx = 0; Gdx < BoundingBox.Num(); Gdx++)
		{
			// Compute BoundingBox
			int32 VertexEnd = VertexStart[Gdx] + VertexCount[Gdx];
			for (int32 Vdx = VertexStart[Gdx]; Vdx < VertexEnd; ++Vdx)
			{
				BoundingBox[Gdx] += FVector(Vertex[Vdx]);
			}
		}
	}

	TArray<FVector> FBoundsFacade::GetCentroids() const
	{
		TArray<FVector> Centroids;

		if (IsValid())
		{
			const TManagedArray<FBox>& BoundingBoxes = BoundingBoxAttribute.Get();

			for (int32 Idx = 0; Idx < BoundingBoxes.Num(); ++Idx)
			{
				Centroids.Add(BoundingBoxes[Idx].GetCenter());
			}
		}
		
		return Centroids;
	}

	TArray<FVector> FBoundsFacade::GetTransformCentroidsInCollectionSpace() const
	{
		TArray<FVector> Centroids;
		if (IsValid())
		{
			const int32 NumTransforms = TransformToGeometryIndexAttribute.Num();
			Centroids.Init(FVector::ZeroVector, NumTransforms);

			GeometryCollection::Facades::FCollectionTransformFacade TransformFacade(ConstCollection);
			if (TransformFacade.IsValid())
			{
				const TArray<FTransform> CollectionSpaceTransforms = TransformFacade.ComputeCollectionSpaceTransforms();

				for (int32 TransformIndex = 0; TransformIndex < NumTransforms; ++TransformIndex)
				{
					const int32 GeoIndex = TransformToGeometryIndexAttribute[TransformIndex];
					if (BoundingBoxAttribute.IsValidIndex(GeoIndex))
					{
						const FBox& GeoBoundingBox = BoundingBoxAttribute[GeoIndex];
						const FTransform CollectionSpaceTransform = CollectionSpaceTransforms[TransformIndex];
						Centroids[TransformIndex] = CollectionSpaceTransform.TransformPosition(GeoBoundingBox.GetCenter());
					}
				}
			}
		}

		return Centroids;
	}

	FBox FBoundsFacade::GetBoundingBoxInCollectionSpace() const
	{
		FBox BoundingBox;
		BoundingBox.Init();

		if (IsValid())
		{
			GeometryCollection::Facades::FCollectionTransformFacade TransformFacade(ConstCollection);
			if (TransformFacade.IsValid())
			{
				TArray<FTransform> CollectionSpaceTransforms = TransformFacade.ComputeCollectionSpaceTransforms();

				for (int32 TransformIndex = 0; TransformIndex < CollectionSpaceTransforms.Num(); ++TransformIndex)
				{
					const int32 GeoIndex = TransformToGeometryIndexAttribute[TransformIndex];
					if (BoundingBoxAttribute.IsValidIndex(GeoIndex))
					{
						const FBox& GeoBoundingBox = BoundingBoxAttribute[GeoIndex];
						const FTransform CollectionSpaceTransform = CollectionSpaceTransforms[TransformIndex];
						const FBox BoundingBoxInCollectionSpace = GeoBoundingBox.TransformBy(CollectionSpaceTransform);
						BoundingBox += BoundingBoxInCollectionSpace;
					}
				}
			}
			else
			{
				const TManagedArray<FBox>& BoundingBoxes = BoundingBoxAttribute.Get();
				for (int32 Idx = 0; Idx < BoundingBoxes.Num(); ++Idx)
				{
					BoundingBox += BoundingBoxes[Idx];
				}
			}
		}

		return BoundingBox;
	}

	TArray<FVector> FBoundsFacade::GetBoundingBoxVertexPositions(const FBox& InBox)
	{
		FVector Min = InBox.Min;
		FVector Max = InBox.Max;
		FVector Extent(Max.X - Min.X, Max.Y - Min.Y, Max.Z - Min.Z);

		TArray<FVector> Vertices;
		Vertices.Add(Min);
		Vertices.Add({ Min.X + Extent.X, Min.Y, Min.Z });
		Vertices.Add({ Min.X + Extent.X, Min.Y + Extent.Y, Min.Z });
		Vertices.Add({ Min.X, Min.Y + Extent.Y, Min.Z });
		Vertices.Add({ Min.X, Min.Y, Min.Z + Extent.Z });
		Vertices.Add({ Min.X + Extent.X, Min.Y, Min.Z + Extent.Z });
		Vertices.Add(Max);
		Vertices.Add({ Min.X, Min.Y + Extent.Y, Min.Z + Extent.Z });

		return Vertices;
	}

	FSphere FBoundsFacade::ComputeBoundingSphere(const FBox& InBoundingBox)
	{
		FSphere BoundingSphere;

		BoundingSphere.Center = InBoundingBox.GetCenter();
		BoundingSphere.W = 0.5 * (InBoundingBox.Max - InBoundingBox.Min).Length();

		return BoundingSphere;
	}

	void FBoundsFacade::ComputeBoundingSphereImpl(const TArray<FVector>& InVertices, FSphere& OutSphere) const
	{
		FBox Box;
		FVector MinIx[3] = { FVector(0), FVector(0), FVector(0) };
		FVector MaxIx[3] = { FVector(0), FVector(0), FVector(0) };

		bool bFirstVertex = true;

		for (int32 Idx = 0; Idx < InVertices.Num(); ++Idx)
		{
			FVector Point = InVertices[Idx];
			if (bFirstVertex)
			{
				// First, find AABB, remembering furthest points in each dir.
				Box.Min = Point;
				Box.Max = Box.Min;

				MinIx[0] = Point;
				MinIx[1] = Point;
				MinIx[2] = Point;

				MaxIx[0] = Point;
				MaxIx[1] = Point;
				MaxIx[2] = Point;
				bFirstVertex = false;
				continue;
			}

			// X //
			if (Point.X < Box.Min.X)
			{
				Box.Min.X = Point.X;
				MinIx[0] = Point;
			}
			else if (Point.X > Box.Max.X)
			{
				Box.Max.X = Point.X;
				MaxIx[0] = Point;
			}

			// Y //
			if (Point.Y < Box.Min.Y)
			{
				Box.Min.Y = Point.Y;
				MinIx[1] = Point;
			}
			else if (Point.Y > Box.Max.Y)
			{
				Box.Max.Y = Point.Y;
				MaxIx[1] = Point;
			}

			// Z //
			if (Point.Z < Box.Min.Z)
			{
				Box.Min.Z = Point.Z;
				MinIx[2] = Point;
			}
			else if (Point.Z > Box.Max.Z)
			{
				Box.Max.Z = Point.Z;
				MaxIx[2] = Point;
			}
		}

		const FVector Extremes[3] = { (MaxIx[0] - MinIx[0]), (MaxIx[1] - MinIx[1]), (MaxIx[2] - MinIx[2]) };

		// Now find extreme points furthest apart, and initial center and radius of sphere.
		double MaxDist2 = 0.0;
		for (int32 i = 0; i < 3; ++i)
		{
			const double TmpDist2 = Extremes[i].SizeSquared();
			if (TmpDist2 > MaxDist2)
			{
				MaxDist2 = TmpDist2;
				OutSphere.Center = (MinIx[i] + (0.5f * Extremes[i]));
				OutSphere.W = 0.f;
			}
		}

		const FVector Extents = FVector(Extremes[0].X, Extremes[1].Y, Extremes[2].Z);

		// radius and radius squared
		double Radius = 0.5f * Extents.GetMax();
		double Radius2 = FMath::Square(Radius);

		// Now check each point lies within this sphere. If not - expand it a bit.
		for (int32 Idx = 0; Idx < InVertices.Num(); ++Idx)
		{
			const FVector CenterToPoint = InVertices[Idx] - OutSphere.Center;
			const double CenterToPoint2 = CenterToPoint.SizeSquared();

			// If this point is outside our current bounding sphere's radius
			if (CenterToPoint2 > Radius2)
			{
				// ..expand radius just enough to include this point.
				const double PointRadius = FMath::Sqrt(CenterToPoint2);
				Radius = 0.5f * (Radius + PointRadius);
				Radius2 = FMath::Square(Radius);

				OutSphere.Center += ((PointRadius - Radius) / PointRadius * CenterToPoint);
			}
		}

		OutSphere.W = Radius;
	}

	void FBoundsFacade::ComputeBoundingSphereImpl2(const TArray<FVector>& InVertices, FSphere& OutSphere) const
	{
		const FBox BoundingBox = GetBoundingBoxInCollectionSpace();

		OutSphere.Center = BoundingBox.GetCenter();
		OutSphere.W = 0.0f;

		for (int32 Idx = 0; Idx < InVertices.Num(); ++Idx)
		{
			const double Dist2 = FVector::DistSquared(InVertices[Idx], OutSphere.Center);
			if (Dist2 > OutSphere.W)
			{
				OutSphere.W = Dist2;
			}
		}
		OutSphere.W = FMath::Sqrt(OutSphere.W);
	}

	void FBoundsFacade::ComputeBoundingSphere(const TArray<FVector>& InVertices, FSphere& OutSphere) const
	{
		FSphere Sphere, Sphere2, BestSphere;

		ComputeBoundingSphereImpl(InVertices, Sphere);
		ComputeBoundingSphereImpl2(InVertices, Sphere2);

		if (Sphere.W < Sphere2.W)
			BestSphere = Sphere;
		else
			BestSphere = Sphere2;

		// Don't use if radius is zero.
		if (BestSphere.W <= 0.f)
		{
			OutSphere = FSphere(0);
		}

		OutSphere = BestSphere;
	}

	FSphere FBoundsFacade::GetBoundingSphereInCollectionSpace()
	{
		FSphere Sphere(0);

		if (VertexAttribute.IsValid())
		{
			GeometryCollection::Facades::FCollectionTransformFacade TransformFacade(ConstCollection);
			if (TransformFacade.IsValid())
			{
				TArray<FTransform> CollectionSpaceTransforms = TransformFacade.ComputeCollectionSpaceTransforms();

				const TManagedArray<FVector3f>& Vertex = VertexAttribute.Get();
				const TManagedArray<int32>& VertexStartArr = VertexStartAttribute.Get();
				const TManagedArray<int32>& VertexCountArr = VertexCountAttribute.Get();

				TArray<FVector> VerticesInCollectionSpace;
				VerticesInCollectionSpace.Reserve(Vertex.Num());

				for (int32 TransformIndex = 0; TransformIndex < CollectionSpaceTransforms.Num(); ++TransformIndex)
				{
					const FTransform CollectionSpaceTransform = CollectionSpaceTransforms[TransformIndex];
					const int32 GeoIndex = TransformToGeometryIndexAttribute[TransformIndex];
					if (GeoIndex != INDEX_NONE)
					{
						const int32 VertexStart = VertexStartArr[GeoIndex];
						const int32 VertexCount = VertexCountArr[GeoIndex];
						for (int32 VertexIdx = VertexStart; VertexIdx < VertexStart + VertexCount; ++VertexIdx)
						{
							VerticesInCollectionSpace.Add(CollectionSpaceTransform.TransformPosition((FVector)Vertex[VertexIdx]));
						}
					}
				}

				ComputeBoundingSphere(VerticesInCollectionSpace, Sphere);
			}
		}

		return Sphere;
	}

	FBox FBoundsFacade::ComputeBoundingBox(const TArray<FVector> InPoints)
	{
		FBox BoundingBox;
		BoundingBox.Init();

		for (int32 PointIdx = 0; PointIdx < InPoints.Num(); ++PointIdx)
		{
			BoundingBox += InPoints[PointIdx];
		}

		return BoundingBox;
	}

}; // GeometryCollection::Facades


