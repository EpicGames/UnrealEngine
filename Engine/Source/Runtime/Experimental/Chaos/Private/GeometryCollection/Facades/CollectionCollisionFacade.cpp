// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GeometryCollection.cpp: FGeometryCollection methods.
=============================================================================*/

#include "GeometryCollection/Facades/CollectionCollisionFacade.h"
#include "GeometryCollection/GeometryCollection.h"

namespace GeometryCollection::Facades
{
	// Attributes
	const FName FCollisionFacade::IsCollisionEnabledAttributeName = "IsCollisionEnabled";

	FCollisionFacade::FCollisionFacade(FManagedArrayCollection& InCollection)
		: ConstCollection(InCollection)
		, Collection(&InCollection)
		, IsCollisionEnabledAttribute(InCollection, IsCollisionEnabledAttributeName, FGeometryCollection::VerticesGroup)
	{
		DefineSchema();
	}

	FCollisionFacade::FCollisionFacade(const FManagedArrayCollection& InCollection)
		: ConstCollection(InCollection)
		, Collection(nullptr)
		, IsCollisionEnabledAttribute(InCollection, IsCollisionEnabledAttributeName, FGeometryCollection::VerticesGroup)
	{
	}

	/** Define the facade */

	void FCollisionFacade::DefineSchema()
	{
		check(!IsConst());
		IsCollisionEnabledAttribute.AddAndFill(false);
	}

	bool FCollisionFacade::IsValid() const
	{
		return IsCollisionEnabledAttribute.IsValid();
	}

	void FCollisionFacade::SetCollisionEnabled(const TArray<int32>& VertexIndices)
	{
		check(!IsConst());
		TManagedArray<bool>& IsCollisionEnabledArray = IsCollisionEnabledAttribute.Modify();
		for (int32 VertexIdx : VertexIndices)
		{
			if (IsCollisionEnabledAttribute.IsValidIndex(VertexIdx))
			{
				IsCollisionEnabledArray[VertexIdx] = true;
			}
		}
	}

	bool FCollisionFacade::IsCollisionEnabled(int32 VertexIndex) const
	{
		if (IsCollisionEnabledAttribute.IsValid() && IsCollisionEnabledAttribute.IsValidIndex(VertexIndex))
		{
			return IsCollisionEnabledAttribute.Get()[VertexIndex];
		}
		return false;
	}
};


