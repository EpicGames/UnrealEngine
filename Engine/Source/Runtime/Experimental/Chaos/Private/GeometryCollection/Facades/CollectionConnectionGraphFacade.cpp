// Copyright Epic Games, Inc. All Rights Reserved.


#include "GeometryCollection/Facades/CollectionConnectionGraphFacade.h"
#include "GeometryCollection/TransformCollection.h"

namespace GeometryCollection::Facades
{

	static const FName ConnectionGroupName = "ConnectionEdge";

// Disable deprecations so that we can initialize the ConnectionsAttribute, though we no longer expect it to be on the Collection
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FCollectionConnectionGraphFacade::FCollectionConnectionGraphFacade(FManagedArrayCollection& InCollection)
		: ConnectionsAttribute(InCollection, "Connections", FTransformCollection::TransformGroup)
		, ConnectionEdgeStartAttribute(InCollection, "ConnectionEdgeStarts", ConnectionGroupName, FTransformCollection::TransformGroup)
		, ConnectionEdgeEndAttribute(InCollection, "ConnectionEdgeEnds", ConnectionGroupName, FTransformCollection::TransformGroup)
		, ConnectionEdgeContactAttribute(InCollection, "ConnectionEdgeContacts", ConnectionGroupName)
#if UE_BUILD_DEBUG
		, ParentAttribute(InCollection, "Parent", FTransformCollection::TransformGroup)
#endif
	{
	}

	FCollectionConnectionGraphFacade::FCollectionConnectionGraphFacade(const FManagedArrayCollection& InCollection)
		: ConnectionsAttribute(InCollection, "Connections", FTransformCollection::TransformGroup)
		, ConnectionEdgeStartAttribute(InCollection, "ConnectionEdgeStarts", ConnectionGroupName, FTransformCollection::TransformGroup)
		, ConnectionEdgeEndAttribute(InCollection, "ConnectionEdgeEnds", ConnectionGroupName, FTransformCollection::TransformGroup)
		, ConnectionEdgeContactAttribute(InCollection, "ConnectionEdgeContacts", ConnectionGroupName)
#if UE_BUILD_DEBUG
		, ParentAttribute(InCollection, "Parent", FTransformCollection::TransformGroup)
#endif
	{
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	bool FCollectionConnectionGraphFacade::IsValid() const
	{
		return ConnectionEdgeStartAttribute.IsValid()
			&& ConnectionEdgeEndAttribute.IsValid();

	}

	void FCollectionConnectionGraphFacade::DefineSchema()
	{
		check(!IsConst());
		ConnectionEdgeStartAttribute.Add();
		ConnectionEdgeEndAttribute.Add();
	}
	
	void FCollectionConnectionGraphFacade::ClearAttributes()
	{
		check(!IsConst());

		ConnectionEdgeStartAttribute.Remove();
		ConnectionEdgeEndAttribute.Remove();
	}

	void FCollectionConnectionGraphFacade::ReserveAdditionalConnections(int32 NumAdditionalConnections)
	{
		check(!IsConst());

		ConnectionEdgeStartAttribute.GetCollection()->Reserve(ConnectionEdgeStartAttribute.Num() + NumAdditionalConnections, ConnectionGroupName);
	}

	void FCollectionConnectionGraphFacade::Connect(int32 BoneA, int32 BoneB)
	{
		if (ConnectionEdgeContactAttribute.IsValid()) // if we have a contact attribute, also set it with a default value
		{
			constexpr float DefaultContact = 1.0f;
			ConnectWithContact(BoneA, BoneB, DefaultContact);
		}

		check(!IsConst());

#if UE_BUILD_DEBUG
		// Expect to always have a parent array and connected bones should always have the same parent
		checkSlow(ParentAttribute.IsValid() && ParentAttribute.Get()[BoneA] == ParentAttribute.Get()[BoneB]);
#endif

		TManagedArray<int32>& Starts = ConnectionEdgeStartAttribute.Modify();
		TManagedArray<int32>& Ends = ConnectionEdgeEndAttribute.Modify();
		int32 NewEdgeIdx = ConnectionEdgeStartAttribute.GetCollection()->AddElements(1, ConnectionGroupName);
		Starts[NewEdgeIdx] = BoneA;
		Ends[NewEdgeIdx] = BoneB;
	}

	void FCollectionConnectionGraphFacade::EnableContactAreas(bool bEnable, float DefaultValue)
	{
		if (bEnable)
		{
			ConnectionEdgeContactAttribute.AddAndFill(DefaultValue);
		}
		else
		{
			ConnectionEdgeContactAttribute.Remove();
		}
	}

	void FCollectionConnectionGraphFacade::ConnectWithContact(int32 BoneA, int32 BoneB, float ContactArea)
	{
		check(!IsConst());

		if (!ConnectionEdgeContactAttribute.IsValid())
		{
			EnableContactAreas(true);
		}

#if UE_BUILD_DEBUG
		// Expect to always have a parent array and connected bones should always have the same parent
		checkSlow(ParentAttribute.IsValid() && ParentAttribute[BoneA] == ParentAttribute[BoneB]);
#endif

		TManagedArray<int32>& Starts = ConnectionEdgeStartAttribute.Modify();
		TManagedArray<int32>& Ends = ConnectionEdgeEndAttribute.Modify();
		TManagedArray<float>& Contacts = ConnectionEdgeContactAttribute.Modify();
		int32 NewEdgeIdx = ConnectionEdgeStartAttribute.GetCollection()->AddElements(1, ConnectionGroupName);
		Starts[NewEdgeIdx] = BoneA;
		Ends[NewEdgeIdx] = BoneB;
		Contacts[NewEdgeIdx] = ContactArea;
	}

	void FCollectionConnectionGraphFacade::ResetConnections()
	{
		check(!IsConst());

		ConnectionEdgeStartAttribute.GetCollection()->EmptyGroup(ConnectionGroupName);
	}

	TPair<int32,int32> FCollectionConnectionGraphFacade::GetConnection(int32 ConnectionIndex) const
	{
		return TPair<int32, int32>(
			ConnectionEdgeStartAttribute[ConnectionIndex],
			ConnectionEdgeEndAttribute[ConnectionIndex]
		);
	}

	float FCollectionConnectionGraphFacade::GetConnectionContactArea(int32 ConnectionIndex) const
	{
		return ConnectionEdgeContactAttribute[ConnectionIndex];
	}

	bool FCollectionConnectionGraphFacade::HasContactAreas() const
	{
		return ConnectionEdgeContactAttribute.IsValid();
	}

	int32 FCollectionConnectionGraphFacade::NumConnections() const
	{
		return ConnectionEdgeStartAttribute.GetConstCollection().NumElements(ConnectionGroupName);
	}

	bool FCollectionConnectionGraphFacade::HasValidConnections() const
	{
		int32 NumTransforms = ConnectionEdgeStartAttribute.GetConstCollection().NumElements(FTransformCollection::TransformGroup);
		const TManagedArray<int32>& Starts = ConnectionEdgeStartAttribute.Get();
		const TManagedArray<int32>& Ends = ConnectionEdgeEndAttribute.Get();
		for (int32 Idx = 0; Idx < Starts.Num(); ++Idx)
		{
			// All indices should be valid indices into the transforms group, and start and end should always be different
			if (Starts[Idx] < 0 || Starts[Idx] >= NumTransforms || Ends[Idx] < 0 || Ends[Idx] >= NumTransforms || Starts[Idx] == Ends[Idx])
			{
				return false;
			}
#if UE_BUILD_DEBUG
			// Expect to always have a parent array and connected bones should always have the same parent
			if (!ParentAttribute.IsValid() || ParentAttribute.Get()[Starts[Idx]] != ParentAttribute.Get()[Ends[Idx]])
			{
				return false;
			}
#endif
		}
		return true;
	}
};


