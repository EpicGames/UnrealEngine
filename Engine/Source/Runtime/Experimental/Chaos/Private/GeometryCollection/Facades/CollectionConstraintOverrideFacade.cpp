// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/Facades/CollectionConstraintOverrideFacade.h"
#include "GeometryCollection/GeometryCollection.h"

namespace GeometryCollection::Facades
{
	//
	// Attributes
	//

	const FName FConstraintOverrideCandidateFacade::GroupName("ConstraintOverrideCandidates");
	const FName FConstraintOverrideCandidateFacade::TargetIndex("TargetIndex");
	const FName FConstraintOverrideCandidateFacade::BoneIndex("BoneIndex");

	const FName FConstraintOverrideTargetFacade::GroupName("ConstraintOverrideTargets");
	const FName FConstraintOverrideTargetFacade::TargetIndex("TargetIndex");
	const FName FConstraintOverrideTargetFacade::TargetPosition("TargetPosition");

	//
	// FConstraintOverrideCandidateFacade
	//

	FConstraintOverrideCandidateFacade::FConstraintOverrideCandidateFacade(FManagedArrayCollection& InCollection)
		: ConstCollection(InCollection)
		, Collection(&InCollection)
		, TargetIndexAttribute(InCollection, TargetIndex, GroupName)
		, BoneIndexAttribute(InCollection, BoneIndex, GroupName)
	{
		DefineSchema();
	}

	FConstraintOverrideCandidateFacade::FConstraintOverrideCandidateFacade(const FManagedArrayCollection& InCollection)
		: ConstCollection(InCollection)
		, Collection(nullptr)
		, TargetIndexAttribute(InCollection, TargetIndex, GroupName)
		, BoneIndexAttribute(InCollection, BoneIndex, GroupName)
	{
	}

	void
	FConstraintOverrideCandidateFacade::DefineSchema()
	{
		check(!IsConst());
		TargetIndexAttribute.Add(ManageArrayAccessor::EPersistencePolicy::MakePersistent, FGeometryCollection::VerticesGroup);
		BoneIndexAttribute.Add(ManageArrayAccessor::EPersistencePolicy::MakePersistent, FGeometryCollection::TransformGroup);
	}

	bool
	FConstraintOverrideCandidateFacade::IsValid() const
	{
		return TargetIndexAttribute.IsValid() && BoneIndexAttribute.IsValid();
	}

	int32
	FConstraintOverrideCandidateFacade::Add(FConstraintOverridesCandidateData& InputData)
	{
		check(!IsConst());
		if (IsValid())
		{
			int32 NewIndex = TargetIndexAttribute.Get().Find(InputData.VertexIndex);
			if (NewIndex == INDEX_NONE)
			{
				NewIndex = TargetIndexAttribute.AddElements(1);
			}
				
			TargetIndexAttribute.Modify()[NewIndex] = InputData.VertexIndex;
			BoneIndexAttribute.Modify()[NewIndex] = InputData.BoneIndex;
			return NewIndex;
		}
		return INDEX_NONE;
	}

	void
	FConstraintOverrideCandidateFacade::Clear()
	{
		check(!IsConst());
		if (IsValid())
		{
			TargetIndexAttribute.GetCollection()->EmptyGroup(GroupName);
		}
	}

	FConstraintOverridesCandidateData
	FConstraintOverrideCandidateFacade::Get(const int32 DataIndex) const
	{
		FConstraintOverridesCandidateData RetVal;
		if (IsValid() && DataIndex >= 0)
		{
			if (TargetIndexAttribute.Num() > DataIndex)
			{
				RetVal.VertexIndex = TargetIndexAttribute.Get()[DataIndex];
			}
			if (BoneIndexAttribute.Num() > DataIndex)
			{
				RetVal.BoneIndex = BoneIndexAttribute.Get()[DataIndex];
			}
		}
		return RetVal;
	}

	//
	// FConstraintOverrideTargetFacade
	//

	FConstraintOverrideTargetFacade::FConstraintOverrideTargetFacade(FManagedArrayCollection& InCollection)
		: ConstCollection(InCollection)
		, Collection(&InCollection)
		, TargetIndexAttribute(InCollection, TargetIndex, GroupName)
		, TargetPositionAttribute(InCollection, TargetPosition, GroupName)
	{
		DefineSchema();
	}

	FConstraintOverrideTargetFacade::FConstraintOverrideTargetFacade(const FManagedArrayCollection& InCollection)
		: ConstCollection(InCollection)
		, Collection(nullptr)
		, TargetIndexAttribute(InCollection, TargetIndex, GroupName)
		, TargetPositionAttribute(InCollection, TargetPosition, GroupName)
	{}

	void
	FConstraintOverrideTargetFacade::DefineSchema()
	{
		check(!IsConst());
		TargetIndexAttribute.Add(ManageArrayAccessor::EPersistencePolicy::MakePersistent, FGeometryCollection::VerticesGroup);
		TargetPositionAttribute.Add(ManageArrayAccessor::EPersistencePolicy::MakePersistent);
	}

	bool
	FConstraintOverrideTargetFacade::IsValid() const
	{
		return TargetIndexAttribute.IsValid() && TargetPositionAttribute.IsValid();
	}

	int32
	FConstraintOverrideTargetFacade::Add(FConstraintOverridesTargetData& InputData)
	{
		check(!IsConst());
		if (IsValid())
		{
			int32 NewIndex = TargetIndexAttribute.Get().Find(InputData.VertexIndex);
			if (NewIndex == INDEX_NONE)
			{
				NewIndex = TargetIndexAttribute.AddElements(1);
			}

			TargetIndexAttribute.Modify()[NewIndex] = InputData.VertexIndex;
			TargetPositionAttribute.Modify()[NewIndex] = InputData.PositionTarget;
			return NewIndex;
		}
		return INDEX_NONE;
	}

	void
	FConstraintOverrideTargetFacade::Clear()
	{
		check(!IsConst());
		if (IsValid())
		{
			TargetIndexAttribute.GetCollection()->EmptyGroup(GroupName);
		}
	}

	FConstraintOverridesTargetData
	FConstraintOverrideTargetFacade::Get(const int32 DataIndex) const
	{
		FConstraintOverridesTargetData RetVal;
		if (IsValid() && DataIndex >= 0)
		{
			if (TargetIndexAttribute.Num() > DataIndex)
			{
				RetVal.VertexIndex = TargetIndexAttribute.Get()[DataIndex];
			}
			if (TargetPositionAttribute.Num() > DataIndex)
			{
				RetVal.PositionTarget = TargetPositionAttribute.Get()[DataIndex];
			}
		}
		return RetVal;
	}

	int32 
	FConstraintOverrideTargetFacade::GetIndex(const int32 DataIndex) const
	{
		FConstraintOverridesTargetData RetVal;
		if (IsValid() && DataIndex >= 0)
		{
			if (TargetIndexAttribute.Num() > DataIndex)
			{
				return TargetIndexAttribute.Get()[DataIndex];
			}
		}
		return INDEX_NONE;
	}

	const FVector3f&
	FConstraintOverrideTargetFacade::GetPosition(const int32 DataIndex) const
	{
		if (IsValid() && DataIndex >= 0)
		{
			if (TargetPositionAttribute.Num() > DataIndex)
			{
				return TargetPositionAttribute.Get()[DataIndex];
			}
		}
		return FVector3f::ZeroVector;
	}

} // namespace GeometryCollection::Facades