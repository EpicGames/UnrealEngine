// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/Facades/CollectionCurveFacade.h"
#include "GeometryCollection/GeometryCollection.h"

namespace GeometryCollection::Facades
{
	// Curves facade  Groups
	const FName FCollectionCurveGeometryFacade::CurvesGroup("Curves");
	const FName FCollectionCurveGeometryFacade::PointsGroup("Points");
	
	// Curves geometry facade attributes
	const FName FCollectionCurveGeometryFacade::CurvePointOffsetsAttribute("PointOffset");
	const FName FCollectionCurveGeometryFacade::PointCurveIndicesAttribute("CurveIndex");
	const FName FCollectionCurveGeometryFacade::GeometryCurveOffsetsAttribute("CurveOffset");
	const FName FCollectionCurveGeometryFacade::CurveGeometryIndicesAttribute("GeometryIndex");
	const FName FCollectionCurveGeometryFacade::PointRestOrientationsAttribute("RestOrientation");
	const FName FCollectionCurveGeometryFacade::PointRestPositionsAttribute("RestPosition");
	const FName FCollectionCurveGeometryFacade::GeometryGroupNamesAttribute("GroupName");
	const FName FCollectionCurveGeometryFacade::GeometryCurveThicknessAttribute("CurveThickness");
	const FName FCollectionCurveGeometryFacade::CurveSourceIndicesAttribute("SourceIndices");
	
	// Curves hierarchy facade attributes
	const FName FCollectionCurveHierarchyFacade::CurveParentIndicesAttribute("ParentIndices");
	const FName FCollectionCurveHierarchyFacade::CurveLodIndicesAttribute("LodIndices");
	
	// Curves attributes facade 
	const FName FCollectionCurveAttributesFacade::CurveCardGroupsAttribute("CardGroups");
	
	FCollectionCurveGeometryFacade::FCollectionCurveGeometryFacade(FManagedArrayCollection& InCollection) :
		ConstCollection(InCollection), Collection(&InCollection),
		PointRestOrientations(InCollection, PointRestOrientationsAttribute, PointsGroup),
		PointRestPositions(InCollection, PointRestPositionsAttribute, PointsGroup),
		CurvePointOffsets(InCollection, CurvePointOffsetsAttribute, CurvesGroup),
		GeometryCurveOffsets(InCollection, GeometryCurveOffsetsAttribute, FGeometryCollection::GeometryGroup),
		PointCurveIndices(InCollection, PointCurveIndicesAttribute, PointsGroup),
		CurveGeometryIndices(InCollection, CurveGeometryIndicesAttribute, CurvesGroup),
		GeometryGroupNames(InCollection, GeometryGroupNamesAttribute, FGeometryCollection::GeometryGroup),
		GeometryCurveThickness(InCollection, GeometryCurveThicknessAttribute, FGeometryCollection::GeometryGroup),
		CurveSourceIndices(InCollection, CurveSourceIndicesAttribute, CurvesGroup)
	{
		DefineSchema();
	}
	
	FCollectionCurveGeometryFacade::FCollectionCurveGeometryFacade(const FManagedArrayCollection& InCollection) :
		ConstCollection(InCollection), Collection(nullptr),
		PointRestOrientations(InCollection, PointRestOrientationsAttribute, PointsGroup),
		PointRestPositions(InCollection, PointRestPositionsAttribute, PointsGroup),
		CurvePointOffsets(InCollection, CurvePointOffsetsAttribute, CurvesGroup),
		GeometryCurveOffsets(InCollection, GeometryCurveOffsetsAttribute, FGeometryCollection::GeometryGroup),
		PointCurveIndices(InCollection, PointCurveIndicesAttribute, PointsGroup),
		CurveGeometryIndices(InCollection, CurveGeometryIndicesAttribute, CurvesGroup),
		GeometryGroupNames(InCollection, GeometryGroupNamesAttribute, FGeometryCollection::GeometryGroup),
		GeometryCurveThickness(InCollection, GeometryCurveThicknessAttribute, FGeometryCollection::GeometryGroup),
		CurveSourceIndices(InCollection, CurveSourceIndicesAttribute, CurvesGroup)
	{
	}
	
	bool FCollectionCurveGeometryFacade::IsValid() const
	{
		return PointRestOrientations.IsValid() && PointRestPositions.IsValid() && 
			CurvePointOffsets.IsValid() && GeometryCurveOffsets.IsValid() && PointCurveIndices.IsValid() &&
					CurveGeometryIndices.IsValid() && GeometryGroupNames.IsValid() &&
						GeometryCurveThickness.IsValid() && CurveSourceIndices.IsValid();
	}
	
	void FCollectionCurveGeometryFacade::DefineSchema()
	{
		check(!IsConst());
		PointRestOrientations.Add();
		PointRestPositions.Add();
		CurvePointOffsets.Add();
		GeometryCurveOffsets.Add();
		PointCurveIndices.Add();
		CurveGeometryIndices.Add();
		GeometryGroupNames.Add();
		GeometryCurveThickness.Add();
		CurveSourceIndices.Add();
	}
	
	void FCollectionCurveGeometryFacade::InitCurvesCollection(const TArray<FVector3f>& InPointRestPositions,
		const TArray<int32>& InCurvePointOffsets, const TArray<int32>& InGeometryCurveOffsets, const TArray<FString>& InGeometryGroupNames,
		const TArray<float>& InGeometryCurveThickness, const TArray<int32>& InCurveSourceIndices)
	{
		if(Collection)
		{
			// Curves group
			Collection->EmptyGroup(CurvesGroup);
			Collection->AddElements(InCurvePointOffsets.Num(), CurvesGroup);

			// Geometry group
			Collection->EmptyGroup(FGeometryCollection::GeometryGroup);
			Collection->AddElements(InGeometryCurveOffsets.Num(), FGeometryCollection::GeometryGroup);

			// Points group
			Collection->EmptyGroup(PointsGroup);
			Collection->AddElements(InPointRestPositions.Num(), PointsGroup);

			// Fill attributes
			SetGeometryCurveOffsets(InGeometryCurveOffsets);
			SetCurvePointOffsets(InCurvePointOffsets);
			SetPointRestPositions(InPointRestPositions);
			SetGeometryGroupNames(InGeometryGroupNames);
			SetGeometryCurveThickness(InGeometryCurveThickness);
			SetCurveSourceIndices(InCurveSourceIndices);

			// Build the geometry collection 
			BuildGeometryCollection();
		}
	}

	void FCollectionCurveGeometryFacade::BuildGeometryCollection()
	{
		if(Collection)
		{
			FGeometryCollection::DefineGeometrySchema(*Collection);

			// Transform group
			Collection->EmptyGroup(FGeometryCollection::TransformGroup);
			Collection->AddElements(GetNumGeometry(), FGeometryCollection::TransformGroup);

			// Vertices group
			Collection->EmptyGroup(FGeometryCollection::VerticesGroup);
			Collection->AddElements(GetNumPoints() * 2, FGeometryCollection::VerticesGroup);

			// Faces group
			Collection->EmptyGroup(FGeometryCollection::FacesGroup);
			Collection->AddElements((GetNumPoints() - GetNumCurves()) * 2, FGeometryCollection::FacesGroup);

			// Material group
			Collection->EmptyGroup(FGeometryCollection::MaterialGroup);

			// Vertices Attributes
			TManagedArray<FVector3f>& VertexPositions = Collection->ModifyAttribute<FVector3f>(FGeometryCollection::VertexPositionAttribute, FGeometryCollection::VerticesGroup);
			TManagedArray<FVector3f>& VertexNormals = Collection->ModifyAttribute<FVector3f>(FGeometryCollection::VertexNormalAttribute, FGeometryCollection::VerticesGroup);
			TManagedArray<FVector3f>& VertexTangentU = Collection->ModifyAttribute<FVector3f>(FGeometryCollection::VertexTangentUAttribute, FGeometryCollection::VerticesGroup);
			TManagedArray<FVector3f>& VertexTangentV = Collection->ModifyAttribute<FVector3f>(FGeometryCollection::VertexTangentVAttribute, FGeometryCollection::VerticesGroup);
			TManagedArray<FLinearColor>& VertexColors = Collection->ModifyAttribute<FLinearColor>(FGeometryCollection::ColorAttribute, FGeometryCollection::VerticesGroup);
			TManagedArray<int32>& VertexTransforms = Collection->ModifyAttribute<int32>(FGeometryCollection::VertexBoneMapAttribute, FGeometryCollection::VerticesGroup);
			
			// Faces Attributes
			TManagedArray<FIntVector>& FaceIndices = Collection->ModifyAttribute<FIntVector>(FGeometryCollection::FaceIndicesAttribute, FGeometryCollection::FacesGroup);
			TManagedArray<bool>& FaceVisibles = Collection->ModifyAttribute<bool>(FGeometryCollection::FaceVisibleAttribute, FGeometryCollection::FacesGroup);

			// Geometry Group
			TManagedArray<int32>& TransformIndex = Collection->ModifyAttribute<int32>(FGeometryCollection::TransformIndexAttribute, FGeometryCollection::GeometryGroup);
			TManagedArray<FBox>& BoundingBox =  Collection->ModifyAttribute<FBox>(FGeometryCollection::BoundingBoxAttribute, FGeometryCollection::GeometryGroup);
			TManagedArray<int32>& VertexStart = Collection->ModifyAttribute<int32>(FGeometryCollection::VertexStartAttribute, FGeometryCollection::GeometryGroup);
			TManagedArray<int32>& VertexCount = Collection->ModifyAttribute<int32>(FGeometryCollection::VertexCountAttribute, FGeometryCollection::GeometryGroup);
			TManagedArray<int32>& FaceStart = Collection->ModifyAttribute<int32>(FGeometryCollection::FaceStartAttribute, FGeometryCollection::GeometryGroup);
			TManagedArray<int32>& FaceCount = Collection->ModifyAttribute<int32>(FGeometryCollection::FaceCountAttribute, FGeometryCollection::GeometryGroup);
			TManagedArray<float>& InnerRadius =Collection->ModifyAttribute<float>(FGeometryCollection::InnerRadiusAttribute, FGeometryCollection::GeometryGroup);
			TManagedArray<float>& OuterRadius = Collection->ModifyAttribute<float>(FGeometryCollection::OuterRadiusAttribute, FGeometryCollection::GeometryGroup);

			// Transform group
			TManagedArray<int32>& BoneGeometry = Collection->ModifyAttribute<int32>(FTransformCollection::GeometryIndexAttribute, FTransformCollection::TransformGroup);
			TManagedArray<FTransform3f>& BoneTransforms = Collection->ModifyAttribute<FTransform3f>(FTransformCollection::TransformAttribute, FTransformCollection::TransformGroup);
			TManagedArray<FString>& BoneNames = Collection->ModifyAttribute<FString>(FTransformCollection::BoneNameAttribute, FTransformCollection::TransformGroup);
			TManagedArray<FLinearColor>& BoneColors = Collection->ModifyAttribute<FLinearColor>(FTransformCollection::BoneColorAttribute , FTransformCollection::TransformGroup);
			TManagedArray<int32>& BoneParent = Collection->ModifyAttribute<int32>(FTransformCollection::ParentAttribute, FTransformCollection::TransformGroup);
			
			const int32 NumGeometry = GetNumGeometry();

			VertexTangentU.Fill(FVector3f::Zero());
			VertexTangentV.Fill(FVector3f::Zero());

			int32 PointOffset = 0;
			int32 CurveOffset = 0;
			for(int32 GeometryIndex = 0; GeometryIndex < NumGeometry; ++GeometryIndex)
			{
				const FLinearColor GeometryColor = FLinearColor::IntToDistinctColor(GeometryIndex, 0.75f, 1.0f, 90.0f);

				// Init geometry/transform attributes
				BoundingBox[GeometryIndex] = FBox(ForceInitToZero);
				TransformIndex[GeometryIndex] = GeometryIndex;
				BoneGeometry[GeometryIndex] = GeometryIndex;
				BoneTransforms[GeometryIndex] = FTransform3f::Identity;
				BoneParent[GeometryIndex] = INDEX_NONE;
				BoneColors[GeometryIndex] = GeometryColor;
				BoneNames[GeometryIndex] = GeometryGroupNames[GeometryIndex];
				VertexStart[GeometryIndex] = PointOffset * 2;
				FaceStart[GeometryIndex] = (PointOffset-CurveOffset) * 2;
				VertexCount[GeometryIndex] = 0;
				FaceCount[GeometryIndex] = 0;
				InnerRadius[GeometryIndex] = GeometryCurveThickness[GeometryIndex];
				OuterRadius[GeometryIndex] = GeometryCurveThickness[GeometryIndex];
				
				for(int32 CurveIndex = CurveOffset, CurveEnd = GeometryCurveOffsets[GeometryIndex];
						  CurveIndex < CurveEnd; ++CurveIndex)
				{
					for (int32 PointIndex = PointOffset, PointEnd = CurvePointOffsets[CurveIndex];
							   PointIndex < PointEnd; ++PointIndex)
					{
						// Build 2 vertices for each points
						const int32 VertexIndex = 2 * PointIndex;

						// Build positions for each curve points
						const FVector3f SideVector = PointRestOrientations[PointIndex].GetAxisX();
						VertexPositions[VertexIndex]   = PointRestPositions[PointIndex] + SideVector * GeometryCurveThickness[GeometryIndex];
						VertexPositions[VertexIndex+1] = PointRestPositions[PointIndex] - SideVector * GeometryCurveThickness[GeometryIndex];

						// Build normals along the up vector
						const FVector3f NormalVector = PointRestOrientations[PointIndex].GetAxisZ();
						VertexNormals[VertexIndex]   = NormalVector;
						VertexNormals[VertexIndex+1] = NormalVector;

						// Build the vertex colors
						VertexColors[VertexIndex] = GeometryColor;
						VertexColors[VertexIndex+1] = GeometryColor;
						
						// Build the vertex transform indices
						VertexTransforms[VertexIndex] = GeometryIndex;
						VertexTransforms[VertexIndex+1] = GeometryIndex;

						// Expand the bounding box
						BoundingBox[GeometryIndex] += FVector(VertexPositions[VertexIndex]);
						BoundingBox[GeometryIndex] += FVector(VertexPositions[VertexIndex+1]);

						VertexCount[GeometryIndex] += 2;

						// Build 2 faces for each edges perpendicular to the up-vector curve
						if(PointIndex < (PointEnd-1))
						{
							const int32 FaceIndex = 2 * (PointIndex - CurveIndex);
							FaceIndices[FaceIndex] = FIntVector(VertexIndex, VertexIndex + 1, VertexIndex + 3);
							FaceIndices[FaceIndex+1] = FIntVector(VertexIndex, VertexIndex + 3, VertexIndex + 2);

							FaceVisibles[FaceIndex] = true;
							FaceVisibles[FaceIndex+1] = true;
							
							FaceCount[GeometryIndex] += 2;
						}
					}
					PointOffset = CurvePointOffsets[CurveIndex];
				}
				CurveOffset = GeometryCurveOffsets[GeometryIndex];
			}
		}
	}
	
	void FCollectionCurveGeometryFacade::UpdateCurveGeometryIndices()
	{
		const int32 NumGeometry = GetNumGeometry();

		int32 CurveOffset = 0;
		for(int32 GeometryIndex = 0; GeometryIndex < NumGeometry; ++GeometryIndex)
		{
			for(int32 CurveIndex = CurveOffset, CurveEnd = GeometryCurveOffsets[GeometryIndex];
					  CurveIndex < CurveEnd; ++CurveIndex)
			{
				CurveGeometryIndices.ModifyAt(CurveIndex,GeometryIndex);
			}
			CurveOffset = GeometryCurveOffsets[GeometryIndex];
		}
	}
	
	void FCollectionCurveGeometryFacade::UpdatePointCurveIndices()
	{
		const int32 NumCurves = GetNumCurves();

		int32 PointOffset = 0;
		for(int32 CurveIndex = 0; CurveIndex < NumCurves; ++CurveIndex)
		{
			for (int32 PointIndex = PointOffset, PointEnd = CurvePointOffsets[CurveIndex];
					   PointIndex < PointEnd; ++PointIndex)
			{
				PointCurveIndices.ModifyAt(PointIndex, CurveIndex);
			}
			PointOffset = CurvePointOffsets[CurveIndex];
		}
	}
	
	void FCollectionCurveGeometryFacade::UpdatePointRestOrientations()
	{
		const int32 NumCurves = GetNumCurves();

		int32 EdgeOffset = 0;
		int32 PointOffset = 0;
		for(int32 CurveIndex = 0; CurveIndex < NumCurves; ++CurveIndex)
		{
			FVector3f TangentPrev = FVector3f(0.,0.,1.), TangentNext = FVector3f::Zero();
			FQuat4f EdgeOrientation = FQuat4f::Identity;

			const int32 NumEdges = (CurvePointOffsets[CurveIndex] - PointOffset)-1;
			for (int32 EdgeIndex = 0; EdgeIndex < NumEdges; ++EdgeIndex)
			{
				TangentPrev = TangentNext;
				TangentNext = (PointRestPositions[PointOffset+EdgeIndex+1]-PointRestPositions[PointOffset+EdgeIndex]).GetSafeNormal();

				EdgeOrientation = (FQuat4f::FindBetweenNormals(TangentPrev,TangentNext) * EdgeOrientation).GetNormalized();
				PointRestOrientations.ModifyAt(EdgeOffset+EdgeIndex, EdgeOrientation);
			}
			PointRestOrientations.ModifyAt(EdgeOffset+NumEdges, EdgeOrientation);
			EdgeOffset += NumEdges+1;
			PointOffset = CurvePointOffsets[CurveIndex];
		}
	}
	
	void FCollectionCurveGeometryFacade::GetCurvePointPositions(const int32 CurveIndex, TArray<FVector3f>& OutPoints) const
	{
		OutPoints.Reset();

		const int32 NumCurves = GetNumCurves();
		if (CurveIndex < 0 || CurveIndex >= NumCurves)
		{
			return;
		}

		const int32 StartPointIndex = (CurveIndex == 0) ? 0 : CurvePointOffsets[CurveIndex-1];
		const int32 EndPointIndex = CurvePointOffsets[CurveIndex];
		const int32 NumPoints = (EndPointIndex - StartPointIndex);
		OutPoints.Reserve(NumPoints);
		for (int32 Index = StartPointIndex; Index < EndPointIndex; ++Index)
		{
			OutPoints.Add(PointRestPositions[Index]);
		}
	}

	void FCollectionCurveGeometryFacade::GetCurveResampledPositions(const int32 CurveIndex, int32 NumSamples, TArray<FVector3f>& OutPoints) const
	{
		OutPoints.Reset();

		if (NumSamples < 2)
		{
			return;
		}

		const int32 NumCurves = GetNumCurves();
		if (CurveIndex < 0 || CurveIndex >= NumCurves)
		{
			return;
		}

		TArray<FVector3f> Positions;
		GetCurvePointPositions(CurveIndex, Positions);
		if (Positions.Num() < 2)
		{
			return;
		}

		// compute lengths
		TArray<double> DistancesAt;
		DistancesAt.Init(0, Positions.Num());

		for (int32 PointIndex = 1; PointIndex < Positions.Num(); ++PointIndex)
		{
			const double SegmentSize = (Positions[PointIndex] - Positions[PointIndex - 1]).Size();
			DistancesAt[PointIndex] = SegmentSize + DistancesAt[PointIndex - 1];
		}

		const double TotalCurveLength = DistancesAt.Last();
		if (TotalCurveLength == 0)
		{
			return;
		}

		OutPoints.Reserve(NumSamples);
		const double SamplingLength = TotalCurveLength / (double)(NumSamples - 1);

		OutPoints.Add(Positions[0]);

		double SampledDistance = SamplingLength;;
		int32 PointIndex = 0;
		while (PointIndex < DistancesAt.Num())
		{
			// find which segment this belong 
			int NextIndex = (PointIndex + 1);
			while (NextIndex < DistancesAt.Num())
			{
				if (SampledDistance < DistancesAt[NextIndex])
				{
					const double RelativeDistance = SampledDistance - DistancesAt[NextIndex - 1];
					const double SegmentLength = DistancesAt[NextIndex] - DistancesAt[NextIndex - 1];
					const double Alpha = (SegmentLength > 0) ? (RelativeDistance / SegmentLength) : 0;
					const FVector3f InterpolatedPos = FMath::Lerp(Positions[NextIndex - 1], Positions[NextIndex], Alpha);
					OutPoints.Add(InterpolatedPos);

					SampledDistance += SamplingLength;
					break;
				}
				++NextIndex;
			}
			PointIndex = (NextIndex - 1);

			// we may have reach the end or gone too far
			if (SampledDistance >= DistancesAt.Last()/* && NumSamples > OutPoints.Num()*/)
			{
				if (OutPoints.Num() < NumSamples)
				{
					OutPoints.Add(Positions.Last());
				}
				break;
			}
		}
	}

	FCollectionCurveHierarchyFacade::FCollectionCurveHierarchyFacade(FManagedArrayCollection& InCollection) :
		ConstCollection(InCollection), Collection(&InCollection),
		CurveParentIndices(InCollection, CurveParentIndicesAttribute, FCollectionCurveGeometryFacade::CurvesGroup),
		CurveLodIndices(InCollection, CurveLodIndicesAttribute, FCollectionCurveGeometryFacade::CurvesGroup)
	{
		DefineSchema();
	}
	
	FCollectionCurveHierarchyFacade::FCollectionCurveHierarchyFacade(const FManagedArrayCollection& InCollection) :
		ConstCollection(InCollection), Collection(nullptr),
		CurveParentIndices(InCollection, CurveParentIndicesAttribute, FCollectionCurveGeometryFacade::CurvesGroup),
		CurveLodIndices(InCollection, CurveLodIndicesAttribute, FCollectionCurveGeometryFacade::CurvesGroup)
	{
	}
	
	bool FCollectionCurveHierarchyFacade::IsValid() const
	{
		return CurveParentIndices.IsValid() && CurveLodIndices.IsValid();
	}
	
	void FCollectionCurveHierarchyFacade::DefineSchema()
	{
		check(!IsConst());
		CurveParentIndices.Add();
		CurveLodIndices.Add();
	}
	
	FCollectionCurveAttributesFacade::FCollectionCurveAttributesFacade(FManagedArrayCollection& InCollection) :
		ConstCollection(InCollection), Collection(&InCollection),
		CurveCardGroups(InCollection, CurveCardGroupsAttribute, FCollectionCurveGeometryFacade::CurvesGroup)
	{
		DefineSchema();
	}
	
	FCollectionCurveAttributesFacade::FCollectionCurveAttributesFacade(const FManagedArrayCollection& InCollection) :
		ConstCollection(InCollection), Collection(nullptr),
		CurveCardGroups(InCollection, CurveCardGroupsAttribute, FCollectionCurveGeometryFacade::CurvesGroup)
	{
	}
	
	bool FCollectionCurveAttributesFacade::IsValid() const
	{
		return CurveCardGroups.IsValid();
	}
	
	void FCollectionCurveAttributesFacade::DefineSchema()
	{
		check(!IsConst());
		CurveCardGroups.Add();
	}
}

