// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/Facades/CollectionExplodedVectorFacade.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/TransformCollection.h"

namespace GeometryCollection::Facades
{
	FCollectionExplodedVectorFacade::FCollectionExplodedVectorFacade(FManagedArrayCollection& InCollection)
		: ExplodedVectorAttribute(InCollection, "ExplodedVector", FGeometryCollection::TransformGroup)
	{
	}

	FCollectionExplodedVectorFacade::FCollectionExplodedVectorFacade(const FManagedArrayCollection& InCollection)
		: ExplodedVectorAttribute(InCollection, "ExplodedVector", FGeometryCollection::TransformGroup)
	{
	}

	bool FCollectionExplodedVectorFacade::IsValid() const
	{
		return ExplodedVectorAttribute.IsValid();
	}

	void FCollectionExplodedVectorFacade::DefineSchema()
	{
		ExplodedVectorAttribute.Add();
	}

	void FCollectionExplodedVectorFacade::UpdateGlobalMatricesWithExplodedVectors(TArray<FMatrix>& InOutGlobalMatrices) const
	{
		if (IsValid())
		{
			const int32 NumMatrices = InOutGlobalMatrices.Num();
			if (NumMatrices > 0)
			{
				const TManagedArray<FVector3f>& ExplodedVectors = ExplodedVectorAttribute.Get();

				if (NumMatrices == ExplodedVectors.Num())
				{
					for (int32 Idx = 0; Idx < NumMatrices; ++Idx)
					{
						InOutGlobalMatrices[Idx] = InOutGlobalMatrices[Idx].ConcatTranslation((FVector)ExplodedVectors[Idx]);
					}
				}
			}
		}
	}

	void FCollectionExplodedVectorFacade::UpdateGlobalMatricesWithExplodedVectors(TArray<FTransform>& InOutGlobalTransforms) const
	{
		if (IsValid())
		{
			const int32 NumMatrices = InOutGlobalTransforms.Num();
			if (NumMatrices > 0)
			{
				const TManagedArray<FVector3f>& ExplodedVectors = ExplodedVectorAttribute.Get();

				if (NumMatrices == ExplodedVectors.Num())
				{
					for (int32 Idx = 0; Idx < NumMatrices; ++Idx)
					{
						InOutGlobalTransforms[Idx].AddToTranslation((FVector)ExplodedVectors[Idx]);
					}
				}
			}
		}
	}
};


