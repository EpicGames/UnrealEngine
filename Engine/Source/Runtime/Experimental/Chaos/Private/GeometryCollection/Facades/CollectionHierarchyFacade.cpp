// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/Facades/CollectionHierarchyFacade.h"
#include "GeometryCollection/TransformCollection.h"

namespace Chaos::Facades
{
	FCollectionHierarchyFacade::FCollectionHierarchyFacade(FManagedArrayCollection& InCollection)
		: ParentAttribute(InCollection, FTransformCollection::ParentAttribute, FTransformCollection::TransformGroup)
	    , ChildrenAttribute(InCollection, FTransformCollection::ChildrenAttribute, FTransformCollection::TransformGroup)
		, LevelAttribute(InCollection, FTransformCollection::LevelAttribute, FTransformCollection::TransformGroup)
	{}

	FCollectionHierarchyFacade::FCollectionHierarchyFacade(const FManagedArrayCollection& InCollection)
		: ParentAttribute(InCollection, FTransformCollection::ParentAttribute, FTransformCollection::TransformGroup)
		, ChildrenAttribute(InCollection, FTransformCollection::ChildrenAttribute, FTransformCollection::TransformGroup)
		, LevelAttribute(InCollection, FTransformCollection::LevelAttribute, FTransformCollection::TransformGroup)
	{}

	bool FCollectionHierarchyFacade::IsValid() const
	{
		return ParentAttribute.IsValid() && ChildrenAttribute.IsValid();
	}

	bool FCollectionHierarchyFacade::HasLevelAttribute() const
	{
		return LevelAttribute.IsValid();
	}

	bool FCollectionHierarchyFacade::IsLevelAttributePersistent() const
	{
		return LevelAttribute.IsValid() && LevelAttribute.IsPersistent();
	}

	int32 FCollectionHierarchyFacade::GetRootIndex() const
	{
		if (IsValid())
		{
			const TManagedArray<int32>& Parents = ParentAttribute.Get();
			for (int32 TransformIndex = 0; TransformIndex < Parents.Num(); ++TransformIndex)
			{
				if (Parents[TransformIndex] == INDEX_NONE)
				{
					return TransformIndex;
				}
			}
			ensureMsgf(Parents.Num() == 0, TEXT("Non-empty collection has invalid hierarchy (no root found)"));
		}
		return INDEX_NONE;
	}

	TArray<int32> FCollectionHierarchyFacade::GetRootIndices() const
	{
		return GetRootIndices(ParentAttribute);
	}

	TArray<int32> FCollectionHierarchyFacade::GetChildrenAsArray(int32 TransformIndex) const
	{
		TArray<int32> ChildrenArray;
		if (ChildrenAttribute.IsValid())
		{
			return ChildrenAttribute.Get()[TransformIndex].Array();
		}
		return ChildrenArray;
	}

	const TSet<int32>* FCollectionHierarchyFacade::FindChildren(int32 TransformIndex) const
	{
		if (ChildrenAttribute.IsValid() && ChildrenAttribute.IsValidIndex(TransformIndex))
		{
			return &ChildrenAttribute[TransformIndex];
		}
		return nullptr;
	}

	int32 FCollectionHierarchyFacade::GetParent(int32 TransformIndex) const
	{
		return ParentAttribute[TransformIndex];
	}

	void FCollectionHierarchyFacade::EnumerateChildTransforms(int32 TransformIndex, TFunctionRef<bool(int32)> ShouldProcess, TFunctionRef<void(int32)> ProcessTransformFunc) const
	{
		checkSlow(ChildrenAttribute.IsValid() && ChildrenAttribute.IsValidIndex(TransformIndex));
		
		if (ShouldProcess(TransformIndex))
		{
			ProcessTransformFunc(TransformIndex);
		}
		else
		{
			for (int32 ChildTransformIndex : ChildrenAttribute[TransformIndex])
			{
				EnumerateChildTransforms(ChildTransformIndex, ShouldProcess, ProcessTransformFunc);
			}
		}
	}

	int32 FCollectionHierarchyFacade::GetInitialLevel(int32 TransformIndex) const
	{
		if (HasLevelAttribute())
		{
			return LevelAttribute.Get()[TransformIndex];
		}
		return INDEX_NONE;
	}

	void FCollectionHierarchyFacade::GenerateLevelAttribute()
	{
		check(!IsConst());
		if (IsValid())
		{
			// level attribute must actually be persistent 
			LevelAttribute.Add(ManageArrayAccessor::EPersistencePolicy::MakePersistent);

			TManagedArray<int32>& Levels = LevelAttribute.Modify();
			const TManagedArray<int32>& Parents = ParentAttribute.Get();

			// In place level computation
			LevelAttribute.Fill(INDEX_NONE);
			for (int32 TransformIndex = 0; TransformIndex < Levels.Num(); TransformIndex++)
			{
				// parent propagation may have already set the value for this transform
				if (Levels[TransformIndex] == INDEX_NONE)
				{
					// first compute the level by walking the parents
					Levels[TransformIndex] = 0;
					int32 ParentIndex = Parents[TransformIndex];
					while (ParentIndex != INDEX_NONE)
					{
						Levels[TransformIndex]++;

						// do we need to go further ? 
						const int32 ParentLevel = Levels[ParentIndex];
						if (ParentLevel != INDEX_NONE)
						{
							Levels[TransformIndex] += ParentLevel;
							break; // early out as the parent level has been computed already
						}

						ParentIndex = Parents[ParentIndex];
					}

					// second propagate to parents if they are still unitialized
					if (Levels[TransformIndex] > 0)
					{
						int32 ParentLevel = Levels[TransformIndex] - 1;
						ParentIndex = Parents[TransformIndex];
						while (ParentIndex != INDEX_NONE && Levels[ParentIndex] == INDEX_NONE)
						{
							Levels[ParentIndex] = ParentLevel;
							ParentLevel--;
							ParentIndex = Parents[ParentIndex];
						}
					}
				}
			}
		}
	}

	TArray<int32> FCollectionHierarchyFacade::GetRootIndices(const TManagedArrayAccessor<int32>& ParentAttribute)
	{
		TArray<int32> Roots;
		if (ParentAttribute.IsValid())
		{
			const TManagedArray<int32>& Parents = ParentAttribute.Get();
			for (int32 Idx = 0; Idx < Parents.Num(); ++Idx)
			{
				if (Parents[Idx] == INDEX_NONE)
				{
					Roots.Add(Idx);
				}
			}
		}

		return Roots;
	}

	TArray<int32> FCollectionHierarchyFacade::GetTransformArrayInDepthFirstOrder() const
	{
		return ComputeTransformIndicesInDepthFirstOrder();
	}

	TArray<int32> FCollectionHierarchyFacade::ComputeTransformIndicesInDepthFirstOrder() const
	{
		TArray<int32> OrderedTransforms;
		if (ParentAttribute.IsValid() && ChildrenAttribute.IsValid())
		{
			const TManagedArray<int32>& Parent = ParentAttribute.Get();
			const TManagedArray<TSet<int32>>& Children = ChildrenAttribute.Get();

			//traverse cluster hierarchy in depth first and record order
			struct FClusterProcessing
			{
				int32 TransformGroupIndex;
				enum
				{
					None,
					VisitingChildren
				} State;

				FClusterProcessing(int32 InIndex) : TransformGroupIndex(InIndex), State(None) {};
			};

			TArray<FClusterProcessing> ClustersToProcess;
			//enqueue all roots
			for (int32 TransformGroupIndex = 0; TransformGroupIndex < Parent.Num(); TransformGroupIndex++)
			{
				if (Parent[TransformGroupIndex] == INDEX_NONE)
				{
					ClustersToProcess.Emplace(TransformGroupIndex);
				}
			}

			OrderedTransforms.Reserve(Parent.Num());

			while (ClustersToProcess.Num())
			{
				FClusterProcessing CurCluster = ClustersToProcess.Pop();
				const int32 ClusterTransformIdx = CurCluster.TransformGroupIndex;
				if (CurCluster.State == FClusterProcessing::VisitingChildren)
				{
					//children already visited
					OrderedTransforms.Add(ClusterTransformIdx);
				}
				else
				{
					if (Children[ClusterTransformIdx].Num())
					{
						CurCluster.State = FClusterProcessing::VisitingChildren;
						ClustersToProcess.Add(CurCluster);

						//order of children doesn't matter as long as all children appear before parent
						for (int32 ChildIdx : Children[ClusterTransformIdx])
						{
							ClustersToProcess.Emplace(ChildIdx);
						}
					}
					else
					{
						OrderedTransforms.Add(ClusterTransformIdx);
					}
				}
			}
		}
		return OrderedTransforms;
	}

	TArray<int32> FCollectionHierarchyFacade::ComputeTransformIndicesInBreadthFirstOrder() const
	{
		TArray<int32> BreadthFirstIndices;
		if (ParentAttribute.IsValid() && ChildrenAttribute.IsValid())
		{
			const TManagedArray<int32>& Parent = ParentAttribute.Get();
			const TManagedArray<TSet<int32>>& Children = ChildrenAttribute.Get();

			// first add all the roots
			BreadthFirstIndices = GetRootIndices();
			BreadthFirstIndices.Reserve(Parent.Num());

			for (int Index = 0; Index < BreadthFirstIndices.Num(); Index++)
			{
				const int32 TransformIndex = BreadthFirstIndices[Index];
				if (Children[TransformIndex].Num())
				{
					for (const int32 ChildTransformIndex : Children[TransformIndex])
					{
						BreadthFirstIndices.Add(ChildTransformIndex);
					}
				}
			}
		}
		return BreadthFirstIndices;
	}

	int32 FCollectionHierarchyFacade::ComputeLevelAttributeMax() const
	{
		if (LevelAttribute.IsValid())
		{
			const TManagedArray<int32>& Level = LevelAttribute.Get();

			int32 LevelMax = -1;
			for (int Index = 0; Index < Level.Num(); Index++)
			{
				if (Level[Index] > LevelMax)
				{
					LevelMax = Level[Index];
				}
			}

			return LevelMax;
		}

		return -1;
	}
}