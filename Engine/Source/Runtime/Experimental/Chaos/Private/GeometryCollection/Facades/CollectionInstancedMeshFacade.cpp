// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/Facades/CollectionInstancedMeshFacade.h"
#include "GeometryCollection/Facades/CollectionHierarchyFacade.h"
#include "GeometryCollection/GeometryCollection.h"

namespace GeometryCollection::Facades
{
	FCollectionInstancedMeshFacade::FCollectionInstancedMeshFacade(FManagedArrayCollection& InCollection)
		: InstancedMeshIndexAttribute(InCollection, "AutoInstanceMeshIndex", FGeometryCollection::TransformGroup)
	{}
	
	FCollectionInstancedMeshFacade::FCollectionInstancedMeshFacade(const FManagedArrayCollection& InCollection)
		: InstancedMeshIndexAttribute(InCollection, "AutoInstanceMeshIndex", FGeometryCollection::TransformGroup)
	{}


	void FCollectionInstancedMeshFacade::DefineSchema()
	{
		InstancedMeshIndexAttribute.AddAndFill(INDEX_NONE);
	}

	bool FCollectionInstancedMeshFacade::IsValid() const
	{
		return InstancedMeshIndexAttribute.IsValid();
	}

	bool FCollectionInstancedMeshFacade::IsConst() const
	{
		return InstancedMeshIndexAttribute.IsConst();
	}

	int32 FCollectionInstancedMeshFacade::GetNumIndices() const
	{
		return IsValid() ? InstancedMeshIndexAttribute.Get().Num(): 0;
	}

	int32 FCollectionInstancedMeshFacade::GetIndex(int32 TransformIndex) const
	{
		return InstancedMeshIndexAttribute.Get()[TransformIndex];
	}

	void FCollectionInstancedMeshFacade::SetIndex(int32 TransformIndex, int32 InstanceMeshIndex)
	{
		check(!IsConst());
		InstancedMeshIndexAttribute.Modify()[TransformIndex] = InstanceMeshIndex;
	}

}