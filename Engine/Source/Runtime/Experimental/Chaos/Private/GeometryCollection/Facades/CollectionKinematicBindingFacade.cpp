// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	FleshCollection.cpp: FFleshCollection methods.
=============================================================================*/

#include "GeometryCollection/Facades/CollectionKinematicBindingFacade.h"
#include "GeometryCollection/GeometryCollection.h"

namespace GeometryCollection::Facades
{
	// Attributes
	const FName FKinematicBindingFacade::KinematicGroup("Kinematics");
	const FName FKinematicBindingFacade::KinematicBoneBindingIndex("Binding");
	const FName FKinematicBindingFacade::KinematicBoneBindingToGroup("BindingToGroup");

	FKinematicBindingFacade::FKinematicBindingFacade(FManagedArrayCollection & InCollection)
		: ConstCollection(InCollection)
		, Collection(&InCollection)
		, KinemaitcBoneBindingAttribute(InCollection, KinematicBoneBindingIndex, KinematicGroup)
		, KinemaitcBoneBindingToGroupAttribute(InCollection, KinematicBoneBindingToGroup, KinematicGroup)
	{}

	FKinematicBindingFacade::FKinematicBindingFacade(const FManagedArrayCollection& InCollection)
		: ConstCollection(InCollection)
		, Collection(nullptr)
		, KinemaitcBoneBindingAttribute(InCollection, KinematicBoneBindingIndex, KinematicGroup)
		, KinemaitcBoneBindingToGroupAttribute(InCollection, KinematicBoneBindingToGroup, KinematicGroup)
	{}

	bool FKinematicBindingFacade::IsValid() const
	{
		return KinemaitcBoneBindingAttribute.IsValid() && KinemaitcBoneBindingToGroupAttribute.IsValid();
	}

	void FKinematicBindingFacade::DefineSchema()
	{
		check(!IsConst());
		FName SelectionGroup1 = FName(GeometryCollection::Facades::FSelectionFacade::BoundGroup.ToString() + "_" + FGeometryCollection::VerticesGroup.ToString());
		KinemaitcBoneBindingAttribute.Add(ManageArrayAccessor::EPersistencePolicy::MakePersistent,SelectionGroup1);
		KinemaitcBoneBindingToGroupAttribute.Add();
	}

	FKinematicBindingFacade::FBindingKey 
	FKinematicBindingFacade::SetBoneBindings(const int32 InBoneIndex, const TArray<int32>& InBoneVerts, const TArray<float>& InBoneWeights)
	{
		check(!IsConst());
		return GeometryCollection::Facades::FSelectionFacade(*Collection).AddSelection(InBoneIndex, InBoneVerts, InBoneWeights, FGeometryCollection::VerticesGroup, FTransformCollection::TransformGroup);
	}

	void FKinematicBindingFacade::GetBoneBindings(const FKinematicBindingFacade::FBindingKey& Key, int32& OutBoneIndex, TArray<int32>& OutBoneVerts, TArray<float>& OutBoneWeights) const
	{
		GeometryCollection::Facades::FSelectionFacade(ConstCollection).GetSelection(Key, OutBoneIndex, OutBoneVerts, OutBoneWeights);
	}


	int32 FKinematicBindingFacade::AddKinematicBinding(const FKinematicBindingFacade::FBindingKey& Key)
	{
		check(!IsConst());
		if (IsValid())
		{
			int32 NewIndex = KinemaitcBoneBindingAttribute.AddElements(1);
			TManagedArray<int32>& KinemaitcBoneBinding = KinemaitcBoneBindingAttribute.Modify();
			TManagedArray<FString>& KinemaitcBoneBindingToGroup = KinemaitcBoneBindingToGroupAttribute.Modify();
			
			KinemaitcBoneBinding[NewIndex] = Key.Index;
			KinemaitcBoneBindingToGroup[NewIndex] = Key.GroupName.ToString();

			return NewIndex;
		}
		return INDEX_NONE;
	}

	FKinematicBindingFacade::FBindingKey FKinematicBindingFacade::GetKinematicBindingKey(int Index) const
	{
		FKinematicBindingFacade::FBindingKey Key;
		if (IsValid())
		{
			const TManagedArray<int32>& KinemaitcBoneBinding = KinemaitcBoneBindingAttribute.Get();
			const TManagedArray<FString>& KinemaitcBoneBindingToGroup = KinemaitcBoneBindingToGroupAttribute.Get();

			if (0 <= Index && Index < ConstCollection.NumElements(KinematicGroup))
			{
				Key.GroupName = FName(KinemaitcBoneBindingToGroup[Index]);
				Key.Index = KinemaitcBoneBinding[Index];
			}

		}
		return Key;
	}
}
