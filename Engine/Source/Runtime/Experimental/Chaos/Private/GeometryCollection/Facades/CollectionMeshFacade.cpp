// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GeometryCollection.cpp: FGeometryCollection methods.
=============================================================================*/

#include "GeometryCollection/Facades/CollectionMeshFacade.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/TransformCollection.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"

namespace GeometryCollection::Facades
{
	FCollectionMeshFacade::FCollectionMeshFacade(FManagedArrayCollection& InCollection)
		: FCollectionUVFacade(InCollection)
		, TransformToGeometryIndexAttribute(InCollection, "TransformToGeometryIndex", FGeometryCollection::TransformGroup)
		, TransformIndexAttribute(InCollection, "TransformIndex", FGeometryCollection::GeometryGroup)
		, VertexAttribute(InCollection, "Vertex", FGeometryCollection::VerticesGroup)
		, TangentUAttribute(InCollection, "TangentU", FGeometryCollection::VerticesGroup)
		, TangentVAttribute(InCollection, "TangentV", FGeometryCollection::VerticesGroup)
		, NormalAttribute(InCollection, "Normal", FGeometryCollection::VerticesGroup)
		, ColorAttribute(InCollection, "Color", FGeometryCollection::VerticesGroup)
		, BoneMapAttribute(InCollection, "BoneMap", FGeometryCollection::VerticesGroup)
		, VertexStartAttribute(InCollection, "VertexStart", FGeometryCollection::GeometryGroup)
		, VertexCountAttribute(InCollection, "VertexCount", FGeometryCollection::GeometryGroup)
		, IndicesAttribute(InCollection, "Indices", FGeometryCollection::FacesGroup)
		, VisibleAttribute(InCollection, "Visible", FGeometryCollection::FacesGroup)
		, MaterialIndexAttribute(InCollection, "MaterialIndex", FGeometryCollection::FacesGroup)
		, MaterialIDAttribute(InCollection, "MaterialID", FGeometryCollection::FacesGroup)
		, InternalAttribute(InCollection, "Internal", FGeometryCollection::FacesGroup)
		, FaceStartAttribute(InCollection, "FaceStart", FGeometryCollection::GeometryGroup)
		, FaceCountAttribute(InCollection, "FaceCount", FGeometryCollection::GeometryGroup)
		, ParentAttribute(InCollection, FTransformCollection::ParentAttribute, FTransformCollection::TransformGroup)
		, TransformAttribute(InCollection, FTransformCollection::TransformAttribute, FTransformCollection::TransformGroup)
	{
	}

	FCollectionMeshFacade::FCollectionMeshFacade(const FManagedArrayCollection& InCollection)
		: FCollectionUVFacade(InCollection)
		, TransformToGeometryIndexAttribute(InCollection, "TransformToGeometryIndex", FGeometryCollection::TransformGroup)
		, TransformIndexAttribute(InCollection, "TransformIndex", FGeometryCollection::GeometryGroup)
		, VertexAttribute(InCollection, "Vertex", FGeometryCollection::VerticesGroup)
		, TangentUAttribute(InCollection, "TangentU", FGeometryCollection::VerticesGroup)
		, TangentVAttribute(InCollection, "TangentV", FGeometryCollection::VerticesGroup)
		, NormalAttribute(InCollection, "Normal", FGeometryCollection::VerticesGroup)
		, ColorAttribute(InCollection, "Color", FGeometryCollection::VerticesGroup)
		, BoneMapAttribute(InCollection, "BoneMap", FGeometryCollection::VerticesGroup)
		, VertexStartAttribute(InCollection, "VertexStart", FGeometryCollection::GeometryGroup)
		, VertexCountAttribute(InCollection, "VertexCount", FGeometryCollection::GeometryGroup)
		, IndicesAttribute(InCollection, "Indices", FGeometryCollection::FacesGroup)
		, VisibleAttribute(InCollection, "Visible", FGeometryCollection::FacesGroup)
		, MaterialIndexAttribute(InCollection, "MaterialIndex", FGeometryCollection::FacesGroup)
		, MaterialIDAttribute(InCollection, "MaterialID", FGeometryCollection::FacesGroup)
		, InternalAttribute(InCollection, "Internal", FGeometryCollection::FacesGroup)
		, FaceStartAttribute(InCollection, "FaceStart", FGeometryCollection::GeometryGroup)
		, FaceCountAttribute(InCollection, "FaceCount", FGeometryCollection::GeometryGroup)
		, ParentAttribute(InCollection, FTransformCollection::ParentAttribute, FTransformCollection::TransformGroup)
		, TransformAttribute(InCollection, FTransformCollection::TransformAttribute, FTransformCollection::TransformGroup)
	{
	}

	bool FCollectionMeshFacade::IsValid() const
	{
		return FCollectionUVFacade::IsValid()
			&& TransformToGeometryIndexAttribute.IsValid()
			&& TransformIndexAttribute.IsValid()
			&& VertexAttribute.IsValid()
			&& TangentUAttribute.IsValid()
			&& TangentVAttribute.IsValid()
			&& NormalAttribute.IsValid()
			&& ColorAttribute.IsValid()
			&& BoneMapAttribute.IsValid()
			&& VertexStartAttribute.IsValid()
			&& VertexCountAttribute.IsValid()
			&& IndicesAttribute.IsValid()
			&& VisibleAttribute.IsValid()
			&& MaterialIndexAttribute.IsValid()
			&& MaterialIDAttribute.IsValid()
			&& InternalAttribute.IsValid()
			&& FaceStartAttribute.IsValid()
			&& FaceCountAttribute.IsValid()
			&& ParentAttribute.IsValid()
			&& TransformAttribute.IsValid()
			;
	}

	void FCollectionMeshFacade::DefineSchema()
	{
		FCollectionUVFacade::DefineSchema();
		TransformToGeometryIndexAttribute.Add();
		TransformIndexAttribute.Add();
		VertexAttribute.Add();
		TangentUAttribute.Add();
		TangentVAttribute.Add();
		NormalAttribute.Add();
		ColorAttribute.Add();
		BoneMapAttribute.Add();
		VertexStartAttribute.Add();
		VertexCountAttribute.Add();
		IndicesAttribute.Add();
		VisibleAttribute.Add();
		MaterialIndexAttribute.Add();
		MaterialIDAttribute.Add();
		InternalAttribute.Add();
		FaceStartAttribute.Add();
		FaceCountAttribute.Add();
		ParentAttribute.Add();
		TransformAttribute.Add();
	}

	const TArray<int32> FCollectionMeshFacade::GetVertexIndices(int32 BoneIdx) const
	{
		const TManagedArray<int32>& TransformToGeometryIndicies = TransformToGeometryIndexAttribute.Get();
		const TManagedArray<int32>& VertexStarts = VertexStartAttribute.Get();
		const TManagedArray<int32>& VertexCounts = VertexCountAttribute.Get();

		TArray<int32> VertexIndices;
		if (TransformToGeometryIndicies.IsValidIndex(BoneIdx))
		{
			const int32 GeometryIndex = TransformToGeometryIndicies[BoneIdx];
			if (VertexStarts.IsValidIndex(GeometryIndex) && VertexCounts.IsValidIndex(GeometryIndex))
			{
				VertexIndices.Reserve(VertexCounts[GeometryIndex]);

				const int32 VertexIndexStart = VertexStarts[GeometryIndex];
				for (int32 Offset = 0; Offset < VertexCounts[GeometryIndex]; ++Offset)
				{
					VertexIndices.Add(VertexIndexStart + Offset);
				}
			}
		}
		return VertexIndices;
	}

	const TArrayView<const FVector3f> FCollectionMeshFacade::GetVertexPositions(int32 BoneIdx) const
	{
		const TManagedArray<int32>& TransformToGeometryIndicies = TransformToGeometryIndexAttribute.Get();
		const TManagedArray<int32>& VertexStarts = VertexStartAttribute.Get();
		const TManagedArray<int32>& VertexCounts = VertexCountAttribute.Get();
		const TManagedArray<FVector3f>& Vertices = VertexAttribute.Get();

		if (TransformToGeometryIndicies.IsValidIndex(BoneIdx))
		{
			const int32 GeometryIndex = TransformToGeometryIndicies[BoneIdx];
			if (VertexStarts.IsValidIndex(GeometryIndex) && VertexCounts.IsValidIndex(GeometryIndex))
			{
				return TArrayView<const FVector3f>(Vertices.GetData() + VertexStarts[GeometryIndex], VertexCounts[GeometryIndex]);
			}
		}
		return {};
	}

	void FCollectionMeshFacade::GetVerticesInCollectionSpace(TArray<FVector3f>& OutPositions) const
	{
		if (IsValid())
		{
			const TManagedArray<FVector3f>& Vertices = VertexAttribute.Get();
			const TManagedArray<int32>& Parents = ParentAttribute.Get();
			const TManagedArray<FTransform3f>& Transforms = TransformAttribute.Get();
			const TManagedArray<int32>& VertexStarts = VertexStartAttribute.Get();
			const TManagedArray<int32>& VertexCounts = VertexCountAttribute.Get();
			const TManagedArray<int32>& TransformIndex = TransformIndexAttribute.Get();

			OutPositions.Reset(Vertices.Num());
			OutPositions.SetNumUninitialized(Vertices.Num());

			TArray<FTransform3f> RootSpaceTransforms;
			int32 Idx = 0;

			GeometryCollectionAlgo::GlobalMatrices(Transforms, Parents, RootSpaceTransforms);

			for (int32 GeometryIdx = 0; GeometryIdx < TransformIndex.Num(); ++GeometryIdx)
			{
				if (TransformIndex.IsValidIndex(GeometryIdx) &&
					VertexStarts.IsValidIndex(GeometryIdx) &&
					VertexCounts.IsValidIndex(GeometryIdx))
				{
					const int32 TransformIdx = TransformIndex[GeometryIdx];

					if (RootSpaceTransforms.IsValidIndex(TransformIdx))
					{
						const int32 GlobalVertexOffset = VertexStarts[GeometryIdx];
						for (int32 LocalVertexIdx = 0; LocalVertexIdx < VertexCounts[GeometryIdx]; ++LocalVertexIdx)
						{
							if (Idx < OutPositions.Num())
							{
								OutPositions[Idx++] = (RootSpaceTransforms[TransformIdx].TransformPosition((Vertices[GlobalVertexOffset + LocalVertexIdx])));
							}

						}
					}
				}
			}

			if (Idx < OutPositions.Num())
			{
				OutPositions.SetNum(Idx);
			}
		}
	}

	const TArrayView<const FIntVector> FCollectionMeshFacade::GetTriangles(int32 BoneIdx) const
	{
		const TManagedArray<int32>& TransformToGeometryIndicies = TransformToGeometryIndexAttribute.Get();
		const TManagedArray<int32>& FaceStarts = FaceStartAttribute.Get();
		const TManagedArray<int32>& FaceCounts = FaceCountAttribute.Get();
		const TManagedArray<FIntVector>& Triangles = IndicesAttribute.Get();

		if (TransformToGeometryIndicies.IsValidIndex(BoneIdx))
		{
			const int32 GeometryIndex = TransformToGeometryIndicies[BoneIdx];
			if (FaceStarts.IsValidIndex(GeometryIndex) && FaceCounts.IsValidIndex(GeometryIndex))
			{
				return TArrayView<const FIntVector>(Triangles.GetData() + FaceStarts[GeometryIndex], FaceCounts[GeometryIndex]);
			}
		}
		return {};
	}

	const TArray<int32> FCollectionMeshFacade::GetFaceIndices(int32 BoneIdx) const
	{
		const TManagedArray<int32>& TransformToGeometryIndicies = TransformToGeometryIndexAttribute.Get();
		const TManagedArray<int32>& FaceStarts = FaceStartAttribute.Get();
		const TManagedArray<int32>& FaceCounts = FaceCountAttribute.Get();

		TArray<int32> FaceIndicies;

		if (TransformToGeometryIndicies.IsValidIndex(BoneIdx))
		{
			const int32 GeometryIndex = TransformToGeometryIndicies[BoneIdx];
			if (FaceStarts.IsValidIndex(GeometryIndex) && FaceCounts.IsValidIndex(GeometryIndex))
			{
				FaceIndicies.Reserve(FaceCounts[GeometryIndex]);

				const int32 FaceIndexStart = FaceStarts[GeometryIndex];
				for (int32 Offset = 0; Offset < FaceCounts[GeometryIndex]; ++Offset)
				{
					FaceIndicies.Add(FaceIndexStart + Offset);
				}
			}
		}
		return FaceIndicies;
	}

	void FCollectionMeshFacade::BakeTransform(int32 TransformIdx, const FTransform& InTransform)
	{
		const TManagedArray<int32>& TransformToGeometryIndices = TransformToGeometryIndexAttribute.Get();
		TManagedArray<FVector3f>& Vertices = VertexAttribute.Modify();
		const TManagedArray<int32>& VertexStarts = VertexStartAttribute.Get();
		const TManagedArray<int32>& VertexCounts = VertexCountAttribute.Get();
		TManagedArray<FVector3f>& Normals = NormalAttribute.Modify();
		TManagedArray<FVector3f>& TangentUs = TangentUAttribute.Modify();
		TManagedArray<FVector3f>& TangentVs = TangentVAttribute.Modify();

		if (TransformToGeometryIndices.IsValidIndex(TransformIdx))
		{
			const int32 GeometryIndex = TransformToGeometryIndices[TransformIdx];
			if (VertexStarts.IsValidIndex(GeometryIndex) && VertexCounts.IsValidIndex(GeometryIndex))
			{
				const int32 VertexIndexStart = VertexStarts[TransformToGeometryIndices[TransformIdx]];

				for (int32 Offset = 0; Offset < VertexCounts[TransformToGeometryIndices[TransformIdx]]; ++Offset)
				{
					const int32 VertexIdx = VertexIndexStart + Offset;
					// VertexIdx is valid indices for te arrays below because those attributes all belong to the Vertices group of the collection
					Vertices[VertexIdx] = (FVector3f)InTransform.TransformPosition((FVector)Vertices[VertexIdx]);
					Normals[VertexIdx] = (FVector3f)InTransform.TransformVector((FVector)Normals[VertexIdx]);
					TangentUs[VertexIdx] = (FVector3f)InTransform.TransformVector((FVector)TangentUs[VertexIdx]);
					TangentVs[VertexIdx] = (FVector3f)InTransform.TransformVector((FVector)TangentVs[VertexIdx]);
				}
			}
		}
	}

	const TArray<int32> FCollectionMeshFacade::GetGeometryGroupIndexArray() const
	{
		TArray<int32> GroupIndexArray;
		const TManagedArray<FVector3f>& Vertices = VertexAttribute.Get();
		const TManagedArray<int32>& VertexStarts = VertexStartAttribute.Get();
		const TManagedArray<int32>& VertexCounts = VertexCountAttribute.Get();
		GroupIndexArray.Init(INDEX_NONE, Vertices.Num());
		for (int32 GroupIdx = 0; GroupIdx < VertexStarts.Num(); ++GroupIdx)
		{
			for (int32 LocalIdx = 0; LocalIdx < VertexCounts[GroupIdx]; ++LocalIdx)
			{
				GroupIndexArray[VertexStarts[GroupIdx] + LocalIdx] = GroupIdx;
			}
		}
		return GroupIndexArray;
	}
};


