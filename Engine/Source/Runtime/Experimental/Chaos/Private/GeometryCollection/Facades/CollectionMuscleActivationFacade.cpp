// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	FleshCollection.cpp: FFleshCollection methods.
=============================================================================*/

#include "GeometryCollection/Facades/CollectionMuscleActivationFacade.h"

#include "Chaos/Utilities.h"
#include "Intersection/IntrRay3Triangle3.h"
#include "TriangleTypes.h"
#include "GeometryCollection/Facades/CollectionMeshFacade.h"

DEFINE_LOG_CATEGORY_STATIC(LogDataflowMuscleActivationFacade, Log, All);

namespace GeometryCollection::Facades
{
	// Attributes
	const FName FMuscleActivationFacade::GroupName("MuscleActivation");
	const FName FMuscleActivationFacade::GeometryGroupIndex("GeometryGroupIndex");
	const FName FMuscleActivationFacade::MuscleActivationElement("MuscleActivationElement");
	const FName FMuscleActivationFacade::OriginInsertionPair("OriginInsertionPair");
	const FName FMuscleActivationFacade::OriginInsertionRestLength("OriginInsertionRestLength");
	const FName FMuscleActivationFacade::FiberDirectionMatrix("FiberDirectionMatrix");
	const FName FMuscleActivationFacade::ContractionVolumeScale("ContractionVolumeScale");
	const FName FMuscleActivationFacade::FiberLengthRatioAtMaxActivation("FiberLengthRatioAtMaxActivation");
	const FName FMuscleActivationFacade::MuscleLengthRatioThresholdForMaxActivation("MuscleLengthRatioThresholdForMaxActivation");
	const FName FMuscleActivationFacade::InflationVolumeScale("InflationVolumeScale");
	const FName FMuscleActivationFacade::FiberStreamline("FiberStreamline");
	const FName FMuscleActivationFacade::FiberStreamlineRestLength("FiberStreamlineRestLength");
	const FName FMuscleActivationFacade::MuscleActivationCurveName("MuscleActivationCurveName");
	const FName FMuscleActivationFacade::LengthActivationCurve("LengthActivationCurve");
	
	FMuscleActivationFacade::FMuscleActivationFacade(FManagedArrayCollection& InCollection)
		: ConstCollection(InCollection)
		, Collection(&InCollection)
		, GeometryGroupIndexAttribute(InCollection, GeometryGroupIndex, GroupName, FGeometryCollection::GeometryGroup)
		, MuscleActivationElementAttribute(InCollection, MuscleActivationElement, GroupName, "Tetrahedral")
		, OriginInsertionPairAttribute(InCollection, OriginInsertionPair, GroupName, FGeometryCollection::VerticesGroup)
		, OriginInsertionRestLengthAttribute(InCollection, OriginInsertionRestLength, GroupName)
		, FiberDirectionMatrixAttribute(InCollection, FiberDirectionMatrix, GroupName)
		, ContractionVolumeScaleAttribute(InCollection, ContractionVolumeScale, GroupName)
		, FiberLengthRatioAtMaxActivationAttribute(InCollection, FiberLengthRatioAtMaxActivation, GroupName)
		, MuscleLengthRatioThresholdForMaxActivationAttribute(InCollection, MuscleLengthRatioThresholdForMaxActivation, GroupName)
		, InflationVolumeScaleAttribute(InCollection, InflationVolumeScale, GroupName)
		, FiberStreamlineAttribute(InCollection, FiberStreamline, GroupName)
		, FiberStreamlineRestLengthAttribute(InCollection, FiberStreamlineRestLength, GroupName)
		, MuscleActivationCurveNameAttribute(InCollection, MuscleActivationCurveName, GroupName)
		, LengthActivationCurveAttribute(InCollection, LengthActivationCurve, GroupName)
	{
		DefineSchema();
	}

	FMuscleActivationFacade::FMuscleActivationFacade(const FManagedArrayCollection& InCollection)
		: ConstCollection(InCollection)
		, Collection(nullptr)
		, GeometryGroupIndexAttribute(InCollection, GeometryGroupIndex, GroupName, FGeometryCollection::GeometryGroup)
		, MuscleActivationElementAttribute(InCollection, MuscleActivationElement, GroupName)
		, OriginInsertionPairAttribute(InCollection, OriginInsertionPair, GroupName)
		, OriginInsertionRestLengthAttribute(InCollection, OriginInsertionRestLength, GroupName)
		, FiberDirectionMatrixAttribute(InCollection, FiberDirectionMatrix, GroupName)
		, ContractionVolumeScaleAttribute(InCollection, ContractionVolumeScale, GroupName)
		, FiberLengthRatioAtMaxActivationAttribute(InCollection, FiberLengthRatioAtMaxActivation, GroupName)
		, MuscleLengthRatioThresholdForMaxActivationAttribute(InCollection, MuscleLengthRatioThresholdForMaxActivation, GroupName)
		, InflationVolumeScaleAttribute(InCollection, InflationVolumeScale, GroupName)
		, FiberStreamlineAttribute(InCollection, FiberStreamline, GroupName)
		, FiberStreamlineRestLengthAttribute(InCollection, FiberStreamlineRestLength, GroupName)
		, MuscleActivationCurveNameAttribute(InCollection, MuscleActivationCurveName, GroupName)
		, LengthActivationCurveAttribute(InCollection, LengthActivationCurve, GroupName)
	{
		
	}

	bool FMuscleActivationFacade::IsValid() const
	{
		return GeometryGroupIndexAttribute.IsValid() && MuscleActivationElementAttribute.IsValid() && OriginInsertionPairAttribute.IsValid() &&
			OriginInsertionRestLengthAttribute.IsValid() && FiberDirectionMatrixAttribute.IsValid() && ContractionVolumeScaleAttribute.IsValid() &&
			FiberLengthRatioAtMaxActivationAttribute.IsValid() && MuscleLengthRatioThresholdForMaxActivationAttribute.IsValid() &&
			InflationVolumeScaleAttribute.IsValid();
	}

	void FMuscleActivationFacade::DefineSchema()
	{
		check(!IsConst());
		GeometryGroupIndexAttribute.Add(ManageArrayAccessor::EPersistencePolicy::MakePersistent, FGeometryCollection::GeometryGroup);
		MuscleActivationElementAttribute.Add(ManageArrayAccessor::EPersistencePolicy::MakePersistent, "Tetrahedral");
		OriginInsertionPairAttribute.Add(ManageArrayAccessor::EPersistencePolicy::MakePersistent, FGeometryCollection::VerticesGroup);
		OriginInsertionRestLengthAttribute.Add();
		FiberDirectionMatrixAttribute.Add();
		ContractionVolumeScaleAttribute.Add();
		FiberLengthRatioAtMaxActivationAttribute.Add(); 
		MuscleLengthRatioThresholdForMaxActivationAttribute.Add();
		InflationVolumeScaleAttribute.Add();
		FiberStreamlineAttribute.Add();
		FiberStreamlineRestLengthAttribute.Add();
		MuscleActivationCurveNameAttribute.Add();
		LengthActivationCurveAttribute.Add();
	}

	int32 FMuscleActivationFacade::AddMuscleActivationData(const FMuscleActivationData& InputData)
	{
		check(!IsConst());
		if (IsValid())
		{
			int32 NewIndex = MuscleActivationElementAttribute.AddElements(1);
			UpdateMuscleActivationData(NewIndex, InputData);
			return NewIndex;
		}
		return INDEX_NONE;
	}

	bool FMuscleActivationFacade::UpdateMuscleActivationData(int32 DataIndex, const FMuscleActivationData& InputData)
	{
		check(!IsConst());
		if (IsValid() && IsValidMuscleIndex(DataIndex))
		{
			GeometryGroupIndexAttribute.Modify()[DataIndex] = InputData.GeometryGroupIndex;
			MuscleActivationElementAttribute.Modify()[DataIndex] = InputData.MuscleActivationElement;
			OriginInsertionPairAttribute.Modify()[DataIndex] = InputData.OriginInsertionPair;
			OriginInsertionRestLengthAttribute.Modify()[DataIndex] = InputData.OriginInsertionRestLength;
			FiberDirectionMatrixAttribute.Modify()[DataIndex] = InputData.FiberDirectionMatrix;
			ContractionVolumeScaleAttribute.Modify()[DataIndex] = InputData.ContractionVolumeScale;
			FiberLengthRatioAtMaxActivationAttribute.Modify()[DataIndex] = InputData.FiberLengthRatioAtMaxActivation;
			MuscleLengthRatioThresholdForMaxActivationAttribute.Modify()[DataIndex] = InputData.MuscleLengthRatioThresholdForMaxActivation;
			InflationVolumeScaleAttribute.Modify()[DataIndex] = InputData.InflationVolumeScale;
			FiberStreamlineAttribute.Modify()[DataIndex] = InputData.FiberStreamline;
			FiberStreamlineRestLengthAttribute.Modify()[DataIndex] = InputData.FiberStreamlineRestLength;
			return true;
		}
		return false;
	}

	FMuscleActivationData FMuscleActivationFacade::GetMuscleActivationData(const int32 DataIndex) const
	{
		FMuscleActivationData ReturnData;
		if (IsValid() && IsValidMuscleIndex(DataIndex))
		{
			ReturnData.GeometryGroupIndex = GeometryGroupIndexAttribute[DataIndex];
			ReturnData.MuscleActivationElement = MuscleActivationElementAttribute[DataIndex];
			ReturnData.OriginInsertionPair = OriginInsertionPairAttribute[DataIndex];
			ReturnData.OriginInsertionRestLength = OriginInsertionRestLengthAttribute[DataIndex];
			ReturnData.FiberDirectionMatrix = FiberDirectionMatrixAttribute[DataIndex];
			ReturnData.ContractionVolumeScale = ContractionVolumeScaleAttribute[DataIndex];
			ReturnData.FiberLengthRatioAtMaxActivation = FiberLengthRatioAtMaxActivationAttribute[DataIndex];
			ReturnData.MuscleLengthRatioThresholdForMaxActivation = MuscleLengthRatioThresholdForMaxActivationAttribute[DataIndex];
			ReturnData.InflationVolumeScale = InflationVolumeScaleAttribute[DataIndex];
			ReturnData.FiberStreamline = FiberStreamlineAttribute[DataIndex];
			ReturnData.FiberStreamlineRestLength = FiberStreamlineRestLengthAttribute[DataIndex];
		}
		return ReturnData;
	}

	int32 FMuscleActivationFacade::MuscleVertexOffset(const int32 MuscleIndex) const
	{
		const TManagedArray<int32>* VertexStart = ConstCollection.FindAttributeTyped<int32>("VertexStart", FGeometryCollection::GeometryGroup);
		if (IsValid() && GeometryGroupIndexAttribute.IsValidIndex(MuscleIndex) && VertexStart->IsValidIndex(GeometryGroupIndexAttribute[MuscleIndex]))
		{
			return (*VertexStart)[GeometryGroupIndexAttribute[MuscleIndex]];
		}
		return 0;
	}
	
	int32 FMuscleActivationFacade::NumMuscleVertices(const int32 MuscleIndex) const
	{
		const TManagedArray<int32>* VertexCount = ConstCollection.FindAttributeTyped<int32>("VertexCount", FGeometryCollection::GeometryGroup);
		if (IsValid() && GeometryGroupIndexAttribute.IsValidIndex(MuscleIndex) && VertexCount->IsValidIndex(GeometryGroupIndexAttribute[MuscleIndex]))
		{
			return (*VertexCount)[GeometryGroupIndexAttribute[MuscleIndex]];
		}
		return 0;
	}

	FString FMuscleActivationFacade::FindMuscleName(const int32 MuscleIndex) const
	{
		const TManagedArray<FString>* BoneName = ConstCollection.FindAttributeTyped<FString>("BoneName", FTransformCollection::TransformGroup);
		const TManagedArray<int32>* TransformIndex = ConstCollection.FindAttributeTyped<int32>("TransformIndex", FGeometryCollection::GeometryGroup);
		if (IsValid() && GeometryGroupIndexAttribute.IsValidIndex(MuscleIndex))
		{
			int32 MuscleGeometryIndex = GeometryGroupIndexAttribute[MuscleIndex];
			if (BoneName && TransformIndex && TransformIndex->IsValidIndex(MuscleGeometryIndex) && BoneName->IsValidIndex((*TransformIndex)[MuscleGeometryIndex]))
			{
				return (*BoneName)[(*TransformIndex)[MuscleGeometryIndex]];
			}
		}
		return FString();
	}

	int32 FMuscleActivationFacade::FindMuscleIndexByName(const FString MuscleName) const
	{
		if (IsValid())
		{
			const TManagedArray<FString>* BoneName = ConstCollection.FindAttributeTyped<FString>("BoneName", FTransformCollection::TransformGroup);
			const TManagedArray<int32>* TransformIndex = ConstCollection.FindAttributeTyped<int32>("TransformIndex", FGeometryCollection::GeometryGroup);
			if (BoneName && TransformIndex)
			{
				int32 MuscleTransformIndex = BoneName->Find(MuscleName);
				if (MuscleTransformIndex >= 0)
				{
					int32 MuscleGeometryIndex = TransformIndex->Find(MuscleTransformIndex);
					if (MuscleGeometryIndex >= 0)
					{
						return GeometryGroupIndexAttribute.Get().Find(MuscleGeometryIndex);
					}
				}
			}
		}
		return INDEX_NONE;
	}

	int32 FMuscleActivationFacade::FindMuscleGeometryIndex(const int32 MuscleIndex) const
	{
		if (IsValid() && GeometryGroupIndexAttribute.IsValidIndex(MuscleIndex))
		{
			return GeometryGroupIndexAttribute[MuscleIndex];
		}
		return INDEX_NONE;
	}

	int32 FMuscleActivationFacade::RemoveInvalidMuscles()
	{
		check(!IsConst());
		TArray<int32> InvalidMuscleIndices;
		if (IsValid())
		{
			for (int32 MuscleIndex = 0; MuscleIndex < GeometryGroupIndexAttribute.Num(); ++MuscleIndex)
			{
				if (GeometryGroupIndexAttribute[MuscleIndex] < 0)
				{
					InvalidMuscleIndices.Add(MuscleIndex);
				}
			}
			Collection->RemoveElements(GroupName, InvalidMuscleIndices);
		}
		return InvalidMuscleIndices.Num();
	}

	bool FMuscleActivationFacade::SetUpMuscleActivation(const TArray<int32>& InOrigin, const TArray<int32>& Insertion, float InContractionVolumeScale)
	{
		check(!IsConst());
		// Vertices and fiber field
		if (InOrigin.Num() == 0)
		{
			UE_LOGF(LogDataflowMuscleActivationFacade, Error, "Muscle activation setup failed: No origins given");
			return false;
		}
		if (Insertion.Num() == 0)
		{
			UE_LOGF(LogDataflowMuscleActivationFacade, Error, "Muscle activation setup failed: No insertions given");
			return false;
		}
		if (!ConstCollection.FindAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup))
		{
			UE_LOGF(LogDataflowMuscleActivationFacade, Error, "Muscle activation setup failed: Collection has no Vertex attribute");
			return false;
		}
		if (!ConstCollection.FindAttribute<FIntVector4>("Tetrahedron", "Tetrahedral"))
		{
			UE_LOGF(LogDataflowMuscleActivationFacade, Error, "Muscle activation setup failed: Collection has no Tetrahedron attribute");
			return false;
		}
		if (!ConstCollection.FindAttribute<FVector3f>("FiberDirection", "Tetrahedral"))
		{
			UE_LOGF(LogDataflowMuscleActivationFacade, Error, "Muscle activation setup failed: Collection has no FiberDirection attribute");
			return false;
		}
		// Check if origins and insertions have overlaps
		int32 OverlapCount = 0;
		const TManagedArray<int32>* TransformIndex = ConstCollection.FindAttribute<int32>("TransformIndex", FGeometryCollection::GeometryGroup);
		const TManagedArray<FString>* BoneName = ConstCollection.FindAttribute<FString>("BoneName", FGeometryCollection::TransformGroup);
		const TManagedArray<int32>* VertexStart = ConstCollection.FindAttribute<int32>("VertexStart", FGeometryCollection::GeometryGroup);
		const TManagedArray<int32>* VertexCount = ConstCollection.FindAttribute<int32>("VertexCount", FGeometryCollection::GeometryGroup);
		TSet<int32> OverlapGeometries;
		for (int32 OriginIdx : InOrigin)
		{
			if (Insertion.Contains(OriginIdx))
			{
				OverlapCount++;
				if (VertexStart && VertexCount)
				{
					for (int32 GeomIdx = 0; GeomIdx < VertexStart->Num(); ++GeomIdx)
					{
						if ((*VertexStart)[GeomIdx] <= OriginIdx && OriginIdx < (*VertexStart)[GeomIdx] + (*VertexCount)[GeomIdx])
						{
							OverlapGeometries.Add(GeomIdx);
							break;
						}
					}
				}
			}
		}
		if (OverlapCount)
		{
			UE_LOGF(LogDataflowMuscleActivationFacade, Error, "Muscle activation setup failed: origins and insertions have %d common indices out of total %d, please check if they are from different sources.", OverlapCount, InOrigin.Num());
			if (TransformIndex && BoneName)
			{
				for (const int32 GeomIdx : OverlapGeometries)
				{
					if (ensure(TransformIndex->IsValidIndex(GeomIdx) && BoneName->IsValidIndex((*TransformIndex)[GeomIdx])))
					{
						UE_LOGF(LogDataflowMuscleActivationFacade, Error, "Overlapped origins and insertions are from transform indexed %d, bone name %ls.", (*TransformIndex)[GeomIdx], *(*BoneName)[(*TransformIndex)[GeomIdx]]);
					}
				}
			}
			return false;
		}
		TArray<int32> Origin = InOrigin;
		const TArray<FVector3f>& Vertices = ConstCollection.FindAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup)->GetConstArray();
		const TArray<FIntVector4>& Elements = ConstCollection.FindAttribute<FIntVector4>("Tetrahedron", "Tetrahedral")->GetConstArray();
		const TArray<FVector3f>& FiberDirections = ConstCollection.FindAttribute<FVector3f>("FiberDirection", "Tetrahedral")->GetConstArray();
		TArray<TArray<int32>> MuscleActivationElements;
		TArray<TArray<int32>> ComponentOrigins; //One origin node per muscle component
		TArray<TArray<int32>> ComponentInsertions; //One insertion node per muscle component
		GeometryCollection::Facades::FCollectionMeshFacade MeshFacade(ConstCollection);
		TArray<int32> ComponentIndex = MeshFacade.GetGeometryGroupIndexArray(); //Vertex index to geometry index
		TMap<int32, int32> ComponentToMuscleIndex; //Component index to muscle index
		TMap<int32, int32> MuscleToComponentIndex; //Muscle index to component index
		Origin.Sort(); //For some order in muscle groups
		for (int32 i = 0; i < Origin.Num(); i++)
		{
			if (!ComponentToMuscleIndex.Contains(ComponentIndex[Origin[i]]))
			{
				ComponentToMuscleIndex.Add(ComponentIndex[Origin[i]], ComponentOrigins.Num());
				MuscleToComponentIndex.Add(ComponentOrigins.Num(), ComponentIndex[Origin[i]]);
				ComponentOrigins.SetNum(ComponentOrigins.Num() + 1);
				ComponentOrigins[ComponentOrigins.Num() - 1].Add(Origin[i]);
			}
			else
			{
				ComponentOrigins[ComponentToMuscleIndex[ComponentIndex[Origin[i]]]].Add(Origin[i]);
			}
		}
		ComponentInsertions.SetNum(ComponentOrigins.Num());
		TSet<int32> InsertionOnlyComponents;
		for (int32 i = 0; i < Insertion.Num(); i++)
		{
			if (!ComponentToMuscleIndex.Contains(ComponentIndex[Insertion[i]]))
			{
				InsertionOnlyComponents.Add(ComponentIndex[Insertion[i]]);
			}
			else
			{
				ComponentInsertions[ComponentToMuscleIndex[ComponentIndex[Insertion[i]]]].Add(Insertion[i]);
			}
		}
		for (int32 ComponentIdx: InsertionOnlyComponents)
		{
			UE_LOGF(LogDataflowMuscleActivationFacade, Warning, "Geometry %d has only insertions but no origins.", ComponentIdx);
		}
		MuscleActivationElements.SetNum(ComponentOrigins.Num());
		for (int32 ElemIdx = 0; ElemIdx < Elements.Num(); ElemIdx++)
		{
			if (ComponentToMuscleIndex.Contains(ComponentIndex[Elements[ElemIdx][0]]))
			{
				MuscleActivationElements[ComponentToMuscleIndex[ComponentIndex[Elements[ElemIdx][0]]]].Add(ElemIdx);
			}
		}
		//Choose one origin-insertion pair per muscle that has largest distance apart within each muscle
		// TODO: painted attribute directed origin-insertion pair
		//use origin-insertion line segment length to estimate activation
		for (int32 MuscleComponentIdx = 0; MuscleComponentIdx < ComponentOrigins.Num(); MuscleComponentIdx++)
		{
			if (ensureMsgf(ComponentOrigins.Num() > 0 && ComponentInsertions.Num() > 0, TEXT("Origin or Insertion missing in the muscle %d"), MuscleComponentIdx))
			{
				FMuscleActivationData MuscleActivationData;
				MuscleActivationData.GeometryGroupIndex = MuscleToComponentIndex[MuscleComponentIdx];
				MuscleActivationData.OriginInsertionRestLength = 0;
				for (int32 OriginIdx : ComponentOrigins[MuscleComponentIdx])
				{
					for (int32 InsertionIdx : ComponentInsertions[MuscleComponentIdx])
					{
						float Dist = (Vertices[OriginIdx] - Vertices[InsertionIdx]).Size();
						if (Dist > MuscleActivationData.OriginInsertionRestLength)
						{
							MuscleActivationData.OriginInsertionPair = FIntVector2(OriginIdx, InsertionIdx);
							MuscleActivationData.OriginInsertionRestLength = Dist;
						}
					}
				}
				MuscleActivationData.MuscleActivationElement = MuscleActivationElements[MuscleComponentIdx];
				MuscleActivationData.FiberDirectionMatrix.SetNum(MuscleActivationElements[MuscleComponentIdx].Num());
				MuscleActivationData.ContractionVolumeScale.SetNum(MuscleActivationElements[MuscleComponentIdx].Num());
				for (int32 LocalElemIdx = 0; LocalElemIdx < MuscleActivationElements[MuscleComponentIdx].Num(); LocalElemIdx++)
				{
					FVector3f V = FiberDirections[MuscleActivationElements[MuscleComponentIdx][LocalElemIdx]];
					// QR decomposition on vvT for orthogonal directions
					FVector3f W = V;
					if (V.X < V.Y)
					{
						W.X += 1.f;
					}
					else
					{
						W.Y += 1.f;
					}
					FVector3f U = (V ^ W).GetSafeNormal();
					W = (U ^ V).GetSafeNormal();
					MuscleActivationData.FiberDirectionMatrix[LocalElemIdx] = Chaos::PMatrix33d(V, W, U);
				}
				MuscleActivationData.ContractionVolumeScale.Init(InContractionVolumeScale, MuscleActivationElements[MuscleComponentIdx].Num());
				AddMuscleActivationData(MuscleActivationData);
			}
		}
		return true;
	}

	void FMuscleActivationFacade::UpdateGlobalMuscleActivationParameters(
		float InGlobalContractionVolumeScale,
		float InGlobalFiberLengthRatioAtMaxActivation,
		float InGlobalMuscleLengthRatioThresholdForMaxActivation,
		float InGlobalInflationVolumeScale)
	{
		check(!IsConst());
		for (int32 MuscleIndex = 0; MuscleIndex < NumMuscles(); ++MuscleIndex)
		{
			UpdateMuscleActivationParameters(
				MuscleIndex,
				InGlobalContractionVolumeScale,
				InGlobalFiberLengthRatioAtMaxActivation,
				InGlobalMuscleLengthRatioThresholdForMaxActivation,
				InGlobalInflationVolumeScale);
		}
	}

	bool FMuscleActivationFacade::UpdateMuscleActivationParameters(
		int32 MuscleIndex,
		float InContractionVolumeScale,
		float InFiberLengthRatioAtMaxActivation,
		float InMuscleLengthRatioThresholdForMaxActivation,
		float InInflationVolumeScale)
	{
		check(!IsConst());
		if (IsValidMuscleIndex(MuscleIndex))
		{
			FMuscleActivationData MuscleActivationData = GetMuscleActivationData(MuscleIndex);
			MuscleActivationData.ContractionVolumeScale.Init(InContractionVolumeScale, MuscleActivationData.MuscleActivationElement.Num());
			MuscleActivationData.FiberLengthRatioAtMaxActivation = InFiberLengthRatioAtMaxActivation;
			MuscleActivationData.MuscleLengthRatioThresholdForMaxActivation = InMuscleLengthRatioThresholdForMaxActivation;
			MuscleActivationData.InflationVolumeScale = InInflationVolumeScale;
			UpdateMuscleActivationData(MuscleIndex, MuscleActivationData);
			return true;
		}
		return false;
	}

	void FMuscleActivationFacade::UpdateGlobalLengthActivationCurve(const Chaos::FLinearCurve& InGlobalLengthActivationCurve)
	{
		check(!IsConst());
		for (int32 MuscleIndex = 0; MuscleIndex < NumMuscles(); ++MuscleIndex)
		{
			UpdateLengthActivationCurve(MuscleIndex, InGlobalLengthActivationCurve);
		}
	}

	void FMuscleActivationFacade::UpdateLengthActivationCurve(int32 MuscleIndex, const Chaos::FLinearCurve& InLengthActivationCurve)
	{
		check(!IsConst());
		if (LengthActivationCurveAttribute.IsValid() && IsValidMuscleIndex(MuscleIndex))
		{
			LengthActivationCurveAttribute.Modify()[MuscleIndex] = InLengthActivationCurve;
		}
	}

	Chaos::FLinearCurve FMuscleActivationFacade::GetLengthActivationCurve(int32 MuscleIndex) const
	{
		if (LengthActivationCurveAttribute.IsValid() && LengthActivationCurveAttribute.IsValidIndex(MuscleIndex))
		{
			return LengthActivationCurveAttribute[MuscleIndex];
		}
		return Chaos::FLinearCurve();
	}

	TArray<TArray<TArray<FVector3f>>> FMuscleActivationFacade::BuildStreamlines(const TArray<int32>& Origin, const TArray<int32>& Insertion,
		int32 NumLinesMultiplier, int32 MaxStreamlineIterations, int32 MaxPointsPerLine)
	{
		TArray<TArray<FVector3f>> LineSegments;
		TArray<TArray<TArray<FVector3f>>> MuscleLineSegments;
		TArray<int32> StreamlineStartElements;
		// Vertices and fiber field
		if (!(ConstCollection.FindAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup)
			&& ConstCollection.FindAttribute<FIntVector4>("Tetrahedron", "Tetrahedral")
			&& ConstCollection.FindAttribute<FVector3f>("FiberDirection", "Tetrahedral")
			&& ConstCollection.FindAttribute<int32>("TetrahedronStart", FGeometryCollection::GeometryGroup)
			&& ConstCollection.FindAttribute<int32>("TetrahedronCount", FGeometryCollection::GeometryGroup)))
		{
			return MuscleLineSegments;
		}
		const TArray<FVector3f>& Vertices = ConstCollection.FindAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup)->GetConstArray();
		const TArray<FIntVector4>& Elements = ConstCollection.FindAttribute<FIntVector4>("Tetrahedron", "Tetrahedral")->GetConstArray();
		const TArray<FVector3f>& FiberDirections = ConstCollection.FindAttribute<FVector3f>("FiberDirection", "Tetrahedral")->GetConstArray();
		const TArray<int32>& TetrahedronStart = ConstCollection.FindAttribute<int32>("TetrahedronStart", FGeometryCollection::GeometryGroup)->GetConstArray();
		const TArray<int32>& TetrahedronCount = ConstCollection.FindAttribute<int32>("TetrahedronCount", FGeometryCollection::GeometryGroup)->GetConstArray();

		ensure(Elements.Num() == FiberDirections.Num());
		StreamlineStartElements.Reset();
		TArray<FVector3f> ModifiedFiberDirections = FiberDirections;
		TArray<TArray<int32>> LocalIndex;
		TArray<TArray<int32>> Mesh;
		Mesh.SetNum(Elements.Num());
		for (int32 i = 0; i < Elements.Num(); i++)
		{
			for (int32 j = 0; j < 4; j++)
			{
				Mesh[i].Add(Elements[i][j]);
			}
		}
		TArray<TArray<int32>> IncidentElementsLocalIndex;
		TArray<TArray<int32>> IncidentElements = Chaos::Utilities::ComputeIncidentElements(Mesh, &LocalIndex);
		TArray<FIntVector2> Faces = Chaos::Utilities::ComputeTetMeshFacePairs(Elements);

		TArray<TArray<int32>> FaceToTet;
		FaceToTet.SetNum(Elements.Num() * 4);
		for (int32 f = 0; f < Faces.Num(); ++f)
		{
			int32 q0 = Faces[f][0] / 4;
			FaceToTet[Faces[f][0]].Add(q0);
			if (Faces[f][1] > -1)
			{
				int32 q1 = Faces[f][1] / 4;
				FaceToTet[Faces[f][0]].Add(q1);
				FaceToTet[Faces[f][1]].Add(q0);
				FaceToTet[Faces[f][1]].Add(q1);
			}
		}
		TArray<bool> bOrigin, bInsertion;
		bOrigin.Init(false, Vertices.Num());
		bInsertion.Init(false, Vertices.Num());
		for (int32 i : Origin)
		{
			bOrigin[i] = true;
		}
		for (int32 i : Insertion)
		{
			bInsertion[i] = true;
		}
		auto bConstrained = [&bOrigin, &bInsertion](int32 i) {
			return bOrigin[i] || bInsertion[i];
		};
		for (int32 i = 0; i < FaceToTet.Num(); ++i)
		{
			if (FaceToTet[i].Num() == 1) //boundary face
			{
				int32 e = FaceToTet[i][0];
				if (bConstrained(Elements[e][0]) || bConstrained(Elements[e][1]) || bConstrained(Elements[e][2]) || bConstrained(Elements[e][3]))
					continue;
				FIntVector3 LocalFace = Chaos::Utilities::TetFace(i % 4);
				FIntVector3 Face = FIntVector3(Elements[e][LocalFace[0]], Elements[e][LocalFace[1]], Elements[e][LocalFace[2]]);
				FVector3f Normal = ((Vertices[Face[1]] - Vertices[Face[0]]) ^ (Vertices[Face[2]] - Vertices[Face[0]])).GetSafeNormal();
				ModifiedFiberDirections[e] = FiberDirections[e] - (FiberDirections[e] | Normal) * Normal;
				ModifiedFiberDirections[e].Normalize();
			}
		}

		TArray<bool> bEndElement;
		bEndElement.Init(false, Elements.Num());
		for (int32 e = 0; e < Elements.Num() / 4; ++e)
		{
			for (int32 ie = 0; ie < 4; ++ie)
			{
				if (bInsertion[Elements[e][ie]])
				{
					bEndElement[e] = true;
				}
			}
		}
		TArray<int32> SampleElements;

		for (int32 i : Origin)
		{
			for (int32 e : IncidentElements[i])
			{
				if (!(bOrigin[Elements[e][0]] && bOrigin[Elements[e][1]] && bOrigin[Elements[e][2]] && bOrigin[Elements[e][3]]))
				{
					SampleElements.AddUnique(e);
				}
			}
		}

		TArray<TArray<FVector3f>> Origin_sampled = Chaos::Utilities::RandomPointsInTet(Vertices, Elements, SampleElements, NumLinesMultiplier);

		for (int32 ij = 0; ij < Origin_sampled.Num(); ++ij)
		{
			for (FVector3f StartPosition : Origin_sampled[ij])
			{
				FVector3f StartDirection = ModifiedFiberDirections[SampleElements[ij]];
				TArray<int32> StartTetCandidate = { SampleElements[ij] };
				TArray<int32> NewStartTetCandidate;
				TArray<FVector3f> CurrentLineSegment;
				CurrentLineSegment.Add(StartPosition);
				int32 Iter = 0;
				FVector3f EndPosition;
				bool bReachEnd = false;
				while (((StartTetCandidate.Num() > 1 && Iter > 0) || (StartTetCandidate.Num() > 0 && Iter == 0)) && Iter < MaxStreamlineIterations)
				{
					if (CurrentLineSegment.Num() > 1 && (CurrentLineSegment[CurrentLineSegment.Num() - 1] - CurrentLineSegment[CurrentLineSegment.Num() - 2]).Size() < 1e-6)
					{
						CurrentLineSegment.Pop();
						break;
					}
					NewStartTetCandidate.Reset();
					bool NonTrivialIntersection = false;
					for (int32 e : StartTetCandidate)
					{
						for (int32 f = 0; f < 4; f++)
						{
							FIntVector3 local_face = Chaos::Utilities::TetFace(f);

							UE::Math::TRay<float> RayIn(StartPosition, StartDirection);
							UE::Geometry::TTriangle3<float> TriangleIn(Vertices[Elements[e][local_face[0]]], Vertices[Elements[e][local_face[1]]], Vertices[Elements[e][local_face[2]]]);
							UE::Geometry::TIntrRay3Triangle3<float> Intersection(RayIn, TriangleIn);
							if (Intersection.Find() && Intersection.IntersectionType == EIntersectionType::Point)
							{
								FVector3f IntersectionPosition = TriangleIn.BarycentricPoint(float(Intersection.TriangleBaryCoords[0]), float(Intersection.TriangleBaryCoords[1]), float(Intersection.TriangleBaryCoords[2]));
								if ((StartPosition - IntersectionPosition).Size() > 1e-6)
								{
									NonTrivialIntersection = true;
									EndPosition = IntersectionPosition;
									StartPosition = EndPosition;
									CurrentLineSegment.Add(EndPosition);
									for (int32 NewTetCandidate : FaceToTet[4 * e + f])
									{
										if (NewTetCandidate != e)
										{
											NewStartTetCandidate.Add(NewTetCandidate);
											StartDirection = ModifiedFiberDirections[NewTetCandidate];
											bReachEnd = bEndElement[NewTetCandidate];
											break;
										}
									}
									NewStartTetCandidate.Add(e);
									break;
								}
							}
						}
						if (NonTrivialIntersection)
						{
							NonTrivialIntersection = false;
							break;
						}
					}
					StartTetCandidate = NewStartTetCandidate;
					Iter++;
					if (bReachEnd)
					{
						LineSegments.Add(CurrentLineSegment);
						StreamlineStartElements.Add(SampleElements[ij]);
						break;
					}
				}
			}
		}

		//Coarsen streamlines
		for (int32 i = 0; i < LineSegments.Num(); ++i)
		{
			if (LineSegments[i].Num() > MaxPointsPerLine)
			{
				float TotalLength = 0;
				for (int32 j = 1; j < LineSegments[i].Num(); ++j)
				{
					TotalLength += (LineSegments[i][j - 1] - LineSegments[i][j]).Size();
				}
				float MinLength = TotalLength / float(MaxPointsPerLine - 1);
				TArray<FVector3f> NewLine;
				NewLine.Add(LineSegments[i][0]);
				float EndLength = 0;
				int32 EndIndex = LineSegments[i].Num() - 1;
				for (int32 j = LineSegments[i].Num() - 1; j >= 0; --j)
				{
					EndLength += (LineSegments[i][j - 1] - LineSegments[i][j]).Size();
					if (EndLength > MinLength)
					{
						EndIndex = j - 1;
						break;
					}
				}
				int32 End = 1;
				float CurrentLength = 0;
				while (End <= EndIndex)
				{
					CurrentLength += (LineSegments[i][End - 1] - LineSegments[i][End]).Size();
					if (CurrentLength > MinLength)
					{
						NewLine.Add(LineSegments[i][End]);
						CurrentLength = 0;
					}
					End += 1;
				}
				if (CurrentLength > 0)
				{
					NewLine.Add(LineSegments[i][End]);
				}
				NewLine.Add(LineSegments[i][LineSegments[i].Num() - 1]);
				LineSegments[i] = NewLine;
			}
		}

		//Split line segments by muscle groups
		TArray<TArray<float>> MuscleLineSegmentRestLength;
		MuscleLineSegments.SetNum(NumMuscles());
		MuscleLineSegmentRestLength.SetNum(NumMuscles());
		TArray<int32> ElementToMuscleIndexArray;
		TArray<int32> GroupIndexToMuscleIndexArray;
		GroupIndexToMuscleIndexArray.Init(INDEX_NONE, TetrahedronStart.Num());
		ElementToMuscleIndexArray.Init(INDEX_NONE, Elements.Num());
		for (int32 MuscleIndex = 0; MuscleIndex < NumMuscles(); ++MuscleIndex)
		{
			FMuscleActivationData MuscleActivationData = GetMuscleActivationData(MuscleIndex);	
			if (0 <= MuscleActivationData.GeometryGroupIndex && MuscleActivationData.GeometryGroupIndex < GroupIndexToMuscleIndexArray.Num())
				GroupIndexToMuscleIndexArray[MuscleActivationData.GeometryGroupIndex] = MuscleIndex;
		}
		for (int32 GroupIndex = 0; GroupIndex < TetrahedronStart.Num(); ++GroupIndex)
		{
			for (int32 LocalIdx = 0; LocalIdx < TetrahedronCount[GroupIndex]; ++LocalIdx)
			{
				ElementToMuscleIndexArray[TetrahedronStart[GroupIndex] + LocalIdx] = GroupIndexToMuscleIndexArray[GroupIndex];
			}
		}
		for (int32 LineIndex = 0; LineIndex < StreamlineStartElements.Num(); ++LineIndex)
		{
			int32 MuscleIndex = ElementToMuscleIndexArray[StreamlineStartElements[LineIndex]];
			if (MuscleIndex >= 0)
			{
				MuscleLineSegments[MuscleIndex].Add(LineSegments[LineIndex]);
				float TotalLength = 0;
				for (int32 j = 1; j < LineSegments[LineIndex].Num(); ++j)
				{
					TotalLength += (LineSegments[LineIndex][j - 1] - LineSegments[LineIndex][j]).Size();
				}
				MuscleLineSegmentRestLength[MuscleIndex].Add(TotalLength);
			}
		}
		
		//Save streamline data
		for (int32 MuscleIndex = 0; MuscleIndex < NumMuscles(); ++MuscleIndex)
		{
			FMuscleActivationData MuscleActivationData = GetMuscleActivationData(MuscleIndex);
			MuscleActivationData.FiberStreamline = MuscleLineSegments[MuscleIndex];
			MuscleActivationData.FiberStreamlineRestLength = MuscleLineSegmentRestLength[MuscleIndex];
			UpdateMuscleActivationData(MuscleIndex, MuscleActivationData);
		}
		return MuscleLineSegments;
	}

	int32 FMuscleActivationFacade::AssignCurveName(const FString& CurveName, const FString& MuscleName)
	{
		check(!IsConst());
		const int32 MuscleIdx = FindMuscleIndexByName(MuscleName);
		if (MuscleActivationCurveNameAttribute.IsValidIndex(MuscleIdx))
		{
			MuscleActivationCurveNameAttribute.Modify()[MuscleIdx] = CurveName;
		}
		return MuscleIdx;
	}

	TArray<int32> FMuscleActivationFacade::FindMuscleIndexByCurveName(const FString& CurveName) const
	{
		TArray<int32> MuscleIndices;
		if (MuscleActivationCurveNameAttribute.IsValid())
		{
			for (int32 MuscleIdx = 0; MuscleIdx < NumMuscles(); ++MuscleIdx)
			{
				//Animation curve name becomes lower case somehow after update, IgnoreCase here
				if (MuscleActivationCurveNameAttribute[MuscleIdx].Equals(CurveName, ESearchCase::IgnoreCase)) 
				{
					MuscleIndices.Add(MuscleIdx);
				}
			}
		}
		return MuscleIndices;
	}
} // end namespace GeometryCollection::Facades

