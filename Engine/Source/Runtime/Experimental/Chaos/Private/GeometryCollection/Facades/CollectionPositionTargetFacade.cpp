// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	FleshCollection.cpp: FFleshCollection methods.
=============================================================================*/

#include "GeometryCollection/Facades/CollectionPositionTargetFacade.h"

namespace GeometryCollection::Facades
{
	// Attributes
	const FName FPositionTargetFacade::GroupName("PositionTargets");
	const FName FPositionTargetFacade::TargetIndex("TargetIndex");
	const FName FPositionTargetFacade::SourceIndex("SourceIndex");
	const FName FPositionTargetFacade::Stiffness("Stiffness");
	const FName FPositionTargetFacade::Damping("Damping");
	const FName FPositionTargetFacade::SourceName("SourceName");
	const FName FPositionTargetFacade::TargetName("TargetName");
	const FName FPositionTargetFacade::TargetWeights("TargetWeights");
	const FName FPositionTargetFacade::SourceWeights("SourceWeights");
	const FName FPositionTargetFacade::IsAnisotropic("IsAnisotropic");
	const FName FPositionTargetFacade::IsZeroRestLength("IsZeroRestLength");

	FPositionTargetFacade::FPositionTargetFacade(FManagedArrayCollection& InCollection, const FName& InVerticesGroup)
		: ConstCollection(InCollection)
		, Collection(&InCollection)
		, VerticesGroup(InVerticesGroup)
		, TargetIndexAttribute(InCollection, TargetIndex, GroupName, InVerticesGroup)
		, SourceIndexAttribute(InCollection, SourceIndex, GroupName, InVerticesGroup)
		, StiffnessAttribute(InCollection, Stiffness, GroupName)
		, DampingAttribute(InCollection, Damping, GroupName)
		, TargetWeightsAttribute(InCollection, TargetWeights, GroupName)
		, SourceWeightsAttribute(InCollection, SourceWeights, GroupName)
		, IsAnisotropicAttribute(InCollection, IsAnisotropic, GroupName)
		, IsZeroRestLengthAttribute(InCollection, IsZeroRestLength, GroupName)
	{
		DefineSchema();
	}

	FPositionTargetFacade::FPositionTargetFacade(const FManagedArrayCollection& InCollection, const FName& InVerticesGroup)
		: ConstCollection(InCollection)
		, Collection(nullptr)
		, VerticesGroup(InVerticesGroup)
		, TargetIndexAttribute(InCollection, TargetIndex, GroupName, InVerticesGroup)
		, SourceIndexAttribute(InCollection, SourceIndex, GroupName, InVerticesGroup)
		, StiffnessAttribute(InCollection, Stiffness, GroupName)
		, DampingAttribute(InCollection, Damping, GroupName)
		, TargetWeightsAttribute(InCollection, TargetWeights, GroupName)
		, SourceWeightsAttribute(InCollection, SourceWeights, GroupName)
		, IsAnisotropicAttribute(InCollection, IsAnisotropic, GroupName)
		, IsZeroRestLengthAttribute(InCollection, IsZeroRestLength, GroupName)
	{
		//DefineSchema();
	}

	bool FPositionTargetFacade::IsValid() const
	{
		// Not checking IsAnisotropicAttribute/IsZeroRestLengthAttribute for backward compatibility in 5.5 and before
		return TargetIndexAttribute.IsValid() && TargetIndexAttribute.GetGroupDependency() ==  VerticesGroup 
			&& SourceIndexAttribute.IsValid() && SourceIndexAttribute.GetGroupDependency() == VerticesGroup 
			&& StiffnessAttribute.IsValid() && DampingAttribute.IsValid() && TargetWeightsAttribute.IsValid() && SourceWeightsAttribute.IsValid();
	}

	void FPositionTargetFacade::DefineSchema()
	{
		check(!IsConst());
		TargetIndexAttribute.Add();
		SourceIndexAttribute.Add();
		StiffnessAttribute.Add();
		DampingAttribute.Add();
		TargetWeightsAttribute.Add();
		SourceWeightsAttribute.Add();
		IsAnisotropicAttribute.Add();
		IsZeroRestLengthAttribute.Add();
	}


	int32 FPositionTargetFacade::AddPositionTarget(const FPositionTargetsData& InputData)
	{
		check(!IsConst());
		if (IsValid())
		{
			int32 NewIndex = TargetIndexAttribute.AddElements(1);
			TargetIndexAttribute.Modify()[NewIndex] = InputData.TargetIndex;
			SourceIndexAttribute.Modify()[NewIndex] = InputData.SourceIndex;
			StiffnessAttribute.Modify()[NewIndex] = InputData.Stiffness;
			DampingAttribute.Modify()[NewIndex] = InputData.Damping;
			TargetWeightsAttribute.Modify()[NewIndex] = InputData.TargetWeights;
			SourceWeightsAttribute.Modify()[NewIndex] = InputData.SourceWeights;
			IsAnisotropicAttribute.Modify()[NewIndex] = InputData.bIsAnisotropic;
			IsZeroRestLengthAttribute.Modify()[NewIndex] = InputData.bIsZeroRestLength;
			return NewIndex;
		}
		return INDEX_NONE;
	}

	FPositionTargetsData FPositionTargetFacade::GetPositionTarget(const int32 DataIndex) const
	{
		FPositionTargetsData ReturnData;
		if (IsValid())
		{
			if (StiffnessAttribute.IsValidIndex(DataIndex))
			{
				ReturnData.Stiffness = StiffnessAttribute.Get()[DataIndex];
			}

			if (DampingAttribute.IsValidIndex(DataIndex))
			{
				ReturnData.Damping = DampingAttribute.Get()[DataIndex];
			}

			if (SourceIndexAttribute.IsValidIndex(DataIndex))
			{
				ReturnData.SourceIndex = SourceIndexAttribute[DataIndex];
			}

			if (TargetIndexAttribute.IsValidIndex(DataIndex))
			{
				ReturnData.TargetIndex = TargetIndexAttribute[DataIndex];
			}

			if (SourceWeightsAttribute.IsValidIndex(DataIndex))
			{
				ReturnData.SourceWeights = SourceWeightsAttribute.Get()[DataIndex];
			}

			if (TargetWeightsAttribute.IsValidIndex(DataIndex))
			{
				ReturnData.TargetWeights = TargetWeightsAttribute.Get()[DataIndex];
			}

			if (IsAnisotropicAttribute.IsValid() && IsAnisotropicAttribute.IsValidIndex(DataIndex))
			{
				ReturnData.bIsAnisotropic = IsAnisotropicAttribute.Get()[DataIndex];
			}
			else
			{
				ReturnData.bIsAnisotropic = false; //backward compatible for 5.5 and before
			}

			if (IsZeroRestLengthAttribute.IsValid() && IsZeroRestLengthAttribute.IsValidIndex(DataIndex))
			{
				ReturnData.bIsZeroRestLength = IsZeroRestLengthAttribute.Get()[DataIndex];
			}
			else
			{
				ReturnData.bIsZeroRestLength = true; //backward compatible for 5.5 and before
			}
		}
		return ReturnData;
	
	}

	int32 FPositionTargetFacade::RemoveInvalidPositionTarget()
	{
		check(!IsConst());
		TArray<int32> InvalidConstraintIndices;
		if (IsValid())
		{
			for (int32 Index = 0; Index < Collection->NumElements(GroupName); ++Index)
			{
				bool bDelete = false;
				for (int32 TargetIdx : TargetIndexAttribute[Index])
				{
					if (TargetIdx < 0)
					{
						bDelete = true;
						InvalidConstraintIndices.Add(Index);
						break;
					}
				}
				if (!bDelete)
				{
					for (int32 SourceIdx : SourceIndexAttribute[Index])
					{
						if (SourceIdx < 0)
						{
							InvalidConstraintIndices.Add(Index);
							break;
						}
					}
				}
			}
			Collection->RemoveElements(GroupName, InvalidConstraintIndices);
		}
		return InvalidConstraintIndices.Num();
	}

	int32 FPositionTargetFacade::RemovePositionTargetBetween(TFunctionRef<bool(const int32 VertexIdx)> IsVertexGroup1, TFunctionRef<bool(const int32 VertexIdx)> IsVertexGroup2)
	{
		check(!IsConst());
		TArray<int32> ConstraintIndicesToRemove;
		if (IsValid())
		{
			for (int32 Index = 0; Index < Collection->NumElements(GroupName); ++Index)
			{
				bool IsTargetGroup1 = true;
				bool IsTargetGroup2 = true;
				for (int32 TargetIdx : TargetIndexAttribute[Index])
				{
					IsTargetGroup1 = IsTargetGroup1 && IsVertexGroup1(TargetIdx);
					IsTargetGroup2 = IsTargetGroup2 && IsVertexGroup2(TargetIdx);
				}
				bool IsSourceGroup1 = true;
				bool IsSourceGroup2 = true;
				for (int32 SourceIdx : SourceIndexAttribute[Index])
				{
					IsSourceGroup1 = IsSourceGroup1 && IsVertexGroup1(SourceIdx);
					IsSourceGroup2 = IsSourceGroup2 && IsVertexGroup2(SourceIdx);
				}
				if ((IsTargetGroup1 && IsSourceGroup2) || (IsTargetGroup2 && IsSourceGroup1))
				{
					ConstraintIndicesToRemove.Add(Index);
				}
			}
			Collection->RemoveElements(GroupName, ConstraintIndicesToRemove);
		}
		return ConstraintIndicesToRemove.Num();
	}
}
