// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/Facades/CollectionRemoveOnBreakFacade.h"
#include "GeometryCollection/GeometryCollection.h"

namespace GeometryCollection::Facades
{
	FRemoveOnBreakData::FRemoveOnBreakData()
		: PackedData(FRemoveOnBreakData::DisabledPackedData)
	{}

	FRemoveOnBreakData::FRemoveOnBreakData(const FVector4f& InPackedData)
		: PackedData(InPackedData)
	{}

	FRemoveOnBreakData::FRemoveOnBreakData(bool bEnable, const FVector2f& BreakTimer, bool bClusterCrumbling, const FVector2f& RemovalTimer)
	{
		SetBreakTimer(BreakTimer.X, BreakTimer.Y);
		SetRemovalTimer(RemovalTimer.X, RemovalTimer.Y);
		SetEnabled(bEnable);
		SetClusterCrumbling(bClusterCrumbling);
	}

	const FVector4f& FRemoveOnBreakData::GetPackedData() const
	{
		return PackedData;
	}

	bool FRemoveOnBreakData::IsEnabled() const
	{
		return PackedData.X >= 0;
	}

	bool FRemoveOnBreakData::GetClusterCrumbling() const
	{
		return IsEnabled() && (PackedData.Z < 0);
	}

	FVector2f FRemoveOnBreakData::GetBreakTimer() const
	{
		return FVector2f(FMath::Abs(PackedData.X), FMath::Abs(PackedData.Y));
	}

	FVector2f FRemoveOnBreakData::GetRemovalTimer() const
	{
		return FVector2f(FMath::Abs(PackedData.Z), FMath::Abs(PackedData.W));
	}

	void FRemoveOnBreakData::SetEnabled(bool bEnable)
	{
		PackedData.X = FMath::Abs(PackedData.X);
		if (!bEnable)
		{
			if (PackedData.X == 0)
			{
				PackedData.X = -FLT_EPSILON;
			}
			else
			{
				PackedData.X = -PackedData.X;
			}
		}
		else
		{
			// restore any correction made
			if (PackedData.X <= FLT_EPSILON)
			{
				PackedData.X = 0;
			}
		}
	}

	void FRemoveOnBreakData::SetClusterCrumbling(bool bClusterCrumbling)
	{
		PackedData.Z = FMath::Abs(PackedData.Z);
		if (bClusterCrumbling)
		{
			if (PackedData.Z == 0)
			{
				PackedData.Z = -FLT_EPSILON;
			}
			else
			{
				PackedData.Z = -PackedData.Z;
			}
		}
		else
		{
			// restore any correction made
			if (PackedData.Z <= FLT_EPSILON)
			{
				PackedData.Z = 0;
			}
		}
	}

	void FRemoveOnBreakData::SetBreakTimer(float MinTime, float MaxTime)
	{
		// enable flag is packed in the break timer data, let's save it first and set it back
		const bool bIsEnabled = IsEnabled();
		PackedData.X = FMath::Min(FMath::Abs(MinTime), FMath::Abs(MaxTime)); // Min break timer
		PackedData.Y = FMath::Max(FMath::Abs(MinTime), FMath::Abs(MaxTime)); // Max break timer
		SetEnabled(bIsEnabled);
	}

	void FRemoveOnBreakData::SetRemovalTimer(float MinTime, float MaxTime)
	{
		// cluster crumbling flag is packed in the removal timer data, let's save it first and set it back
		const bool bClusterCrumbling = GetClusterCrumbling();
		PackedData.Z = FMath::Min(FMath::Abs(MinTime), FMath::Abs(MaxTime)); // Min removal timer
		PackedData.W = FMath::Max(FMath::Abs(MinTime), FMath::Abs(MaxTime)); // Max removal timer
		SetClusterCrumbling(bClusterCrumbling);
	}

	//////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	FCollectionRemoveOnBreakFacade::FCollectionRemoveOnBreakFacade(FManagedArrayCollection& InCollection)
		: RemoveOnBreakAttribute(InCollection, "RemoveOnBreak", FGeometryCollection::TransformGroup)
	{}

	FCollectionRemoveOnBreakFacade::FCollectionRemoveOnBreakFacade(const FManagedArrayCollection& InCollection)
		: RemoveOnBreakAttribute(InCollection, "RemoveOnBreak", FGeometryCollection::TransformGroup)
	{}

	void FCollectionRemoveOnBreakFacade::DefineSchema()
	{
		check(!IsConst());
		RemoveOnBreakAttribute.AddAndFill(FRemoveOnBreakData::DisabledPackedData);
	}

	void FCollectionRemoveOnBreakFacade::RemoveSchema()
	{
		check(!IsConst());
		RemoveOnBreakAttribute.Remove();
	}

	bool FCollectionRemoveOnBreakFacade::IsValid() const
	{
		return RemoveOnBreakAttribute.IsValid();
	}

	bool FCollectionRemoveOnBreakFacade::IsConst() const
	{
		return RemoveOnBreakAttribute.IsConst();
	}

	void FCollectionRemoveOnBreakFacade::SetFromIndexArray(const TArray<int32>& Indices, const FRemoveOnBreakData& Data)
	{
		check(!IsConst());
		if (IsValid())
		{
			TManagedArray<FVector4f>& RemoveOnBreak = RemoveOnBreakAttribute.Modify();
			for (int32 Index : Indices)
			{
				RemoveOnBreak[Index] = Data.GetPackedData();
			}
		}
	}

	void FCollectionRemoveOnBreakFacade::SetToAll(const FRemoveOnBreakData& Data)
	{
		check(!IsConst());
		RemoveOnBreakAttribute.Fill(Data.GetPackedData());
	}

	const FRemoveOnBreakData FCollectionRemoveOnBreakFacade::GetData(int32 Index) const
	{
		if (IsValid())
		{
			return RemoveOnBreakAttribute.Get()[Index];
		}
		return FRemoveOnBreakData::DisabledPackedData;
	}
}
