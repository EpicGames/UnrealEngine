// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GeometryCollection.cpp: FGeometryCollection methods.
=============================================================================*/

#include "GeometryCollection/Facades/CollectionRenderingFacade.h"
#include "GeometryCollection/Facades/CollectionVertexBoneWeightsFacade.h"
#include "GeometryCollection/GeometryCollection.h"
#include "Generators/CapsuleGenerator.h"

namespace GeometryCollection::Facades
{

	FRenderingFacade::FRenderingFacade(FManagedArrayCollection& InCollection)
		: ConstCollection(InCollection)
		, Collection(&InCollection)
		, VertexAttribute(InCollection, "Vertex", FGeometryCollection::VerticesGroup)
		, VertexToGeometryIndexAttribute(InCollection, "GeometryIndex", FGeometryCollection::VerticesGroup, FGeometryCollection::GeometryGroup)
		, VertexSelectionAttribute(InCollection, "SelectionState", FGeometryCollection::VerticesGroup)
		, VertexHitProxyIndexAttribute(InCollection, "HitIndex", FGeometryCollection::VerticesGroup)
		, VertexNormalAttribute(InCollection, "Normal", FGeometryCollection::VerticesGroup)
		, VertexColorAttribute(InCollection, "Color", FGeometryCollection::VerticesGroup)
		, VertexUVAttribute(InCollection, "UV", FGeometryCollection::VerticesGroup)
		, IndicesAttribute(InCollection, "Indices", FGeometryCollection::FacesGroup, FGeometryCollection::VerticesGroup)
		, MaterialIDAttribute(InCollection, "MaterialID", FGeometryCollection::FacesGroup)
		, TriangleSectionAttribute(InCollection, "Sections", FGeometryCollection::MaterialGroup)
		, MaterialPathAttribute(InCollection, "MaterialPath", FGeometryCollection::MaterialGroup)
		, GeometryNameAttribute(InCollection, "Name", FGeometryCollection::GeometryGroup)
		, GeometryTransformAttribute(InCollection, "Transform", FGeometryCollection::GeometryGroup)
		, GeometryHitProxyIndexAttribute(InCollection, "HitIndex", FGeometryCollection::GeometryGroup)
		, VertexStartAttribute(InCollection, "VertexStart", FGeometryCollection::GeometryGroup)
		, VertexCountAttribute(InCollection, "VertexCount", FGeometryCollection::GeometryGroup)
		, IndicesStartAttribute(InCollection, "IndicesStart", FGeometryCollection::GeometryGroup)
		, IndicesCountAttribute(InCollection, "IndicesCount", FGeometryCollection::GeometryGroup)
		, MaterialStartAttribute(InCollection, "MaterialsStart", FGeometryCollection::GeometryGroup, FGeometryCollection::MaterialGroup)
		, MaterialCountAttribute(InCollection, "MaterialsCount", FGeometryCollection::GeometryGroup)
		, GeometrySelectionAttribute(InCollection, "SelectionState", FGeometryCollection::GeometryGroup)
		, BoneWeightsAttribute(InCollection, FVertexBoneWeightsFacade::BoneWeightsAttributeName, FGeometryCollection::VerticesGroup)
		, BoneIndicesAttribute(InCollection, FVertexBoneWeightsFacade::BoneIndicesAttributeName, FGeometryCollection::VerticesGroup)
	{}

	FRenderingFacade::FRenderingFacade(const FManagedArrayCollection& InCollection)
		: ConstCollection(InCollection)
		, Collection(nullptr)
		, VertexAttribute(InCollection, "Vertex", FGeometryCollection::VerticesGroup)
		, VertexToGeometryIndexAttribute(InCollection, "GeometryIndex", FGeometryCollection::VerticesGroup, FGeometryCollection::GeometryGroup)
		, VertexSelectionAttribute(InCollection, "SelectionState", FGeometryCollection::VerticesGroup)
		, VertexHitProxyIndexAttribute(InCollection, "HitIndex", FGeometryCollection::VerticesGroup)
		, VertexNormalAttribute(InCollection, "Normal", FGeometryCollection::VerticesGroup)
		, VertexColorAttribute(InCollection, "Color", FGeometryCollection::VerticesGroup)
		, VertexUVAttribute(InCollection, "UV", FGeometryCollection::VerticesGroup)
		, IndicesAttribute(InCollection, "Indices", FGeometryCollection::FacesGroup, FGeometryCollection::VerticesGroup)
		, MaterialIDAttribute(InCollection, "MaterialID", FGeometryCollection::FacesGroup)
		, TriangleSectionAttribute(InCollection, "Sections", FGeometryCollection::MaterialGroup)
		, MaterialPathAttribute(InCollection, "MaterialPath", FGeometryCollection::MaterialGroup)
		, GeometryNameAttribute(InCollection, "Name", FGeometryCollection::GeometryGroup)
		, GeometryTransformAttribute(InCollection, "Transform", FGeometryCollection::GeometryGroup)
		, GeometryHitProxyIndexAttribute(InCollection, "HitIndex", FGeometryCollection::GeometryGroup)
		, VertexStartAttribute(InCollection, "VertexStart", FGeometryCollection::GeometryGroup, FGeometryCollection::VerticesGroup)
		, VertexCountAttribute(InCollection, "VertexCount", FGeometryCollection::GeometryGroup)
		, IndicesStartAttribute(InCollection, "IndicesStart", FGeometryCollection::GeometryGroup, FGeometryCollection::FacesGroup)
		, IndicesCountAttribute(InCollection, "IndicesCount", FGeometryCollection::GeometryGroup)
		, MaterialStartAttribute(InCollection, "MaterialsStart", FGeometryCollection::GeometryGroup, FGeometryCollection::MaterialGroup)
		, MaterialCountAttribute(InCollection, "MaterialsCount", FGeometryCollection::GeometryGroup)
		, GeometrySelectionAttribute(InCollection, "SelectionState", FGeometryCollection::GeometryGroup)
		, BoneWeightsAttribute(InCollection, FVertexBoneWeightsFacade::BoneWeightsAttributeName, FGeometryCollection::VerticesGroup)
		, BoneIndicesAttribute(InCollection, FVertexBoneWeightsFacade::BoneIndicesAttributeName, FGeometryCollection::VerticesGroup)
	{}

	//
	//  Initialization
	//

	void FRenderingFacade::DefineSchema()
	{
		check(!IsConst());
		VertexAttribute.Add();
		VertexSelectionAttribute.Add();
		VertexToGeometryIndexAttribute.Add();
		VertexHitProxyIndexAttribute.Add();
		VertexNormalAttribute.Add();
		VertexColorAttribute.Add();
		VertexUVAttribute.Add();
		IndicesAttribute.Add();
		MaterialIDAttribute.Add();
		TriangleSectionAttribute.Add();
		MaterialPathAttribute.Add();
		GeometryNameAttribute.Add();
		GeometryTransformAttribute.Add();
		GeometryHitProxyIndexAttribute.Add();
		VertexStartAttribute.Add();
		VertexCountAttribute.Add();
		IndicesStartAttribute.Add();
		IndicesCountAttribute.Add();
		MaterialStartAttribute.Add();
		MaterialCountAttribute.Add();
		GeometrySelectionAttribute.Add();
	}

	bool FRenderingFacade::CanRenderSurface( ) const
	{
		return  IsValid() && GetIndices().Num() && GetVertices().Num();
	}

	bool FRenderingFacade::IsValid( ) const
	{
		return VertexAttribute.IsValid() && VertexToGeometryIndexAttribute.IsValid() &&
			VertexSelectionAttribute.IsValid() && VertexHitProxyIndexAttribute.IsValid() &&
			IndicesAttribute.IsValid() &&
			MaterialIDAttribute.IsValid() && 
			TriangleSectionAttribute.IsValid() &&
			GeometryNameAttribute.IsValid() && GeometryHitProxyIndexAttribute.IsValid() &&
			VertexStartAttribute.IsValid() && VertexCountAttribute.IsValid() &&
			IndicesStartAttribute.IsValid() && IndicesCountAttribute.IsValid() &&
			GeometrySelectionAttribute.IsValid() && VertexColorAttribute.IsValid() &&
			VertexNormalAttribute.IsValid() &&
			VertexUVAttribute.IsValid() &&
			MaterialPathAttribute.IsValid();
	}

	int32 FRenderingFacade::NumTriangles() const
	{
		if (IsValid())
		{
			return GetIndices().Num();
		}
			 
		return 0;
	}

	static void AddPointGeometryHelper(const FVector3f& InPoint, TArrayView<FVector3f> Vertices, int32 VertexStart, TArrayView<FIntVector> Tris)
	{
		constexpr float Extension = 1.f;

		// Add vertices
		Vertices[0] = FVector3f(InPoint) + FVector3f(-Extension, 0.f, 0.f);
		Vertices[1] = FVector3f(InPoint) + FVector3f(Extension, 0.f, 0.f);
		FVector3f Dir = Vertices[1] - Vertices[0];
		FVector3f DirAdd = Dir;
		DirAdd.Y += Extension * 0.1f;
		FVector3f OrthogonalDir = (Dir ^ DirAdd).GetSafeNormal();
		OrthogonalDir *= 0.02f;
		Vertices[2] = FVector3f(InPoint) + OrthogonalDir;

		Vertices[3] = FVector3f(InPoint) + FVector3f(0.f, -Extension, 0.f);
		Vertices[4] = FVector3f(InPoint) + FVector3f(0.f, Extension, 0.f);
		Dir = Vertices[4] - Vertices[0];
		DirAdd = Dir;
		DirAdd.X += Extension * 0.1f;
		OrthogonalDir = (Dir ^ DirAdd).GetSafeNormal();
		OrthogonalDir *= 0.02f;
		Vertices[5] = FVector3f(InPoint) + OrthogonalDir;

		Vertices[6] = FVector3f(InPoint) + FVector3f(0.f, 0.f, -Extension);
		Vertices[7] = FVector3f(InPoint) + FVector3f(0.f, 0.f, Extension);
		Dir = Vertices[7] - Vertices[0];
		DirAdd = Dir;
		DirAdd.X += Extension * 0.1f;
		OrthogonalDir = (Dir ^ DirAdd).GetSafeNormal();
		OrthogonalDir *= 0.02f;
		Vertices[8] = FVector3f(InPoint) + OrthogonalDir;

		constexpr float SmallExtension = .5f * Extension;

		Vertices[9] = FVector3f(InPoint) + FVector3f(-SmallExtension, 0.f, 0.f);
		Vertices[10] = FVector3f(InPoint) + FVector3f(SmallExtension, 0.f, 0.f);
		Vertices[11] = FVector3f(InPoint) + FVector3f(0.f, -SmallExtension, 0.f);
		Vertices[12] = FVector3f(InPoint) + FVector3f(0.f, SmallExtension, 0.f);
		Vertices[13] = FVector3f(InPoint) + FVector3f(0.f, 0.f, -SmallExtension);
		Vertices[14] = FVector3f(InPoint) + FVector3f(0.f, 0.f, SmallExtension);

		Tris[0] = FIntVector( VertexStart + 0,  VertexStart + 1,  VertexStart + 2  );
		Tris[1] = FIntVector( VertexStart + 3,  VertexStart + 4,  VertexStart + 5  );
		Tris[2] = FIntVector( VertexStart + 6,  VertexStart + 7,  VertexStart + 8  );
		Tris[3] = FIntVector( VertexStart + 13, VertexStart + 9,  VertexStart + 14 );
		Tris[4] = FIntVector( VertexStart + 14, VertexStart + 10, VertexStart + 13 );
		Tris[5] = FIntVector( VertexStart + 9,  VertexStart + 11, VertexStart + 10 );
		Tris[6] = FIntVector( VertexStart + 10, VertexStart + 12, VertexStart + 9  );
		Tris[7] = FIntVector( VertexStart + 13, VertexStart + 11, VertexStart + 12 );
		Tris[8] = FIntVector( VertexStart + 12, VertexStart + 11, VertexStart + 14 );
	}

	void FRenderingFacade::AddPoint(const FVector3f& InPoint)
	{
		check(!IsConst());
		if (IsValid())
		{
			constexpr int32 NumVertices = 15;
			constexpr int32 NumTriangles = 9;

			TManagedArray<FVector3f>& Vertices = VertexAttribute.Modify();
			TManagedArray<FIntVector>& Indices = IndicesAttribute.Modify();

			int32 IndicesStart = IndicesAttribute.AddElements(NumTriangles);
			int32 VertexStart = VertexAttribute.AddElements(NumVertices);
			TArrayView<FIntVector> TriangleView(Indices.GetData() + IndicesStart, NumTriangles);
			TArrayView<FVector3f> VertexView(Vertices.GetData() + VertexStart, NumVertices);
			AddPointGeometryHelper(InPoint, VertexView, VertexStart, TriangleView);
		}
	}

	void FRenderingFacade::AddPoints(TArray<FVector3f>&& InPoints)
	{
		check(!IsConst());
		if (IsValid() && !InPoints.IsEmpty())
		{
			constexpr int32 NumVerticesPerPoint = 15;
			constexpr int32 NumTrianglesPerPoint = 9;
			TManagedArray<FVector3f>& Vertices = VertexAttribute.Modify();
			TManagedArray<FIntVector>& Indices = IndicesAttribute.Modify();

			int32 IndicesStart = IndicesAttribute.AddElements(NumTrianglesPerPoint * InPoints.Num());
			int32 VertexStart = VertexAttribute.AddElements(NumVerticesPerPoint * InPoints.Num());

			for (int32 Idx = 0; Idx < InPoints.Num(); ++Idx)
			{
				TArrayView<FIntVector> TriangleView(Indices.GetData() + IndicesStart + Idx * NumTrianglesPerPoint, NumTrianglesPerPoint);
				int32 LocalVertexStart = VertexStart + Idx * NumVerticesPerPoint;
				TArrayView<FVector3f> VertexView(Vertices.GetData() + LocalVertexStart, NumVerticesPerPoint);
				AddPointGeometryHelper(InPoints[Idx], VertexView, LocalVertexStart, TriangleView);
			}
		}
	}

	void FRenderingFacade::AddTriangle(const Chaos::FTriangle& InTriangle)
	{
		check(!IsConst());
		if (IsValid())
		{
			auto CollectionVert = [](const Chaos::FVec3& V) { return FVector3f(float(V.X), float(V.Y), float(V.Z)); };

			TManagedArray<FVector3f>& Vertices = VertexAttribute.Modify();
			TManagedArray<FIntVector>& Indices = IndicesAttribute.Modify();
			
			int32 IndicesStart = IndicesAttribute.AddElements(1);
			int32 VertexStart = VertexAttribute.AddElements(3);

			Indices[IndicesStart] = FIntVector(VertexStart, VertexStart + 1, VertexStart + 2);
			Vertices[VertexStart] = CollectionVert(InTriangle[0]);
			Vertices[VertexStart + 1] = CollectionVert(InTriangle[1]);
			Vertices[VertexStart + 2] = CollectionVert(InTriangle[2]);
		}
	}

	void FRenderingFacade::AddBox(const FBox& InBox)
	{
		AddBox((FVector3f)InBox.Min, (FVector3f)InBox.Max);
	}

	void FRenderingFacade::AddBox(const FVector3f& InMinVertex, const FVector3f& InMaxVertex)
	{
		constexpr const int32 NumCorners = 8;
		const FVector3f Corners[NumCorners] =
		{
			FVector3f(InMinVertex.X, InMinVertex.Y, InMinVertex.Z), // -X / -Y / -Z
			FVector3f(InMaxVertex.X, InMinVertex.Y, InMinVertex.Z), // +X / -Y / -Z
			FVector3f(InMaxVertex.X, InMaxVertex.Y, InMinVertex.Z), // +X / +Y / -Z
			FVector3f(InMinVertex.X, InMaxVertex.Y, InMinVertex.Z), // -X / +Y / -Z
			FVector3f(InMinVertex.X, InMinVertex.Y, InMaxVertex.Z), // -X / -Y / +Z
			FVector3f(InMaxVertex.X, InMinVertex.Y, InMaxVertex.Z), // +X / -Y / +Z
			FVector3f(InMaxVertex.X, InMaxVertex.Y, InMaxVertex.Z), // +X / +Y / +Z
			FVector3f(InMinVertex.X, InMaxVertex.Y, InMaxVertex.Z), // -X / +Y / +Z
		};

		constexpr int32 NumVertices = 24; // 6 faces with each 4 vertices
		constexpr int32 NumTriangles = 12;

		TArray<FVector3f> Vertices; Vertices.AddUninitialized(NumVertices);
		TArray<FVector3f> Normals; Normals.AddUninitialized(NumVertices);
		TArray<FIntVector> Tris; Tris.AddUninitialized(NumTriangles);

		// +Z face
		Vertices[0] = Corners[0]; Normals[0] = FVector3f::ZAxisVector;
		Vertices[1] = Corners[1]; Normals[1] = FVector3f::ZAxisVector;
		Vertices[2] = Corners[2]; Normals[2] = FVector3f::ZAxisVector;
		Vertices[3] = Corners[3]; Normals[3] = FVector3f::ZAxisVector;
		Tris[0] = FIntVector{ 0, 1, 3 };
		Tris[1] = FIntVector{ 1, 2, 3 };

		// +Z face
		Vertices[4] = Corners[4]; Normals[4] = -FVector3f::ZAxisVector;
		Vertices[5] = Corners[5]; Normals[5] = -FVector3f::ZAxisVector;
		Vertices[6] = Corners[6]; Normals[6] = -FVector3f::ZAxisVector;
		Vertices[7] = Corners[7]; Normals[7] = -FVector3f::ZAxisVector;
		Tris[2] = FIntVector{ 5, 4, 7 };
		Tris[3] = FIntVector{ 5, 7, 6 };

		// +X face
		Vertices[8]  = Corners[1]; Normals[8]  = FVector3f::XAxisVector;
		Vertices[9]  = Corners[2]; Normals[9]  = FVector3f::XAxisVector;
		Vertices[10] = Corners[6]; Normals[10] = FVector3f::XAxisVector;
		Vertices[11] = Corners[5]; Normals[11] = FVector3f::XAxisVector;
		Tris[4] = FIntVector{ 8, 11, 10 };
		Tris[5] = FIntVector{ 8, 10, 9 };

		// -X face
		Vertices[12] = Corners[0]; Normals[12] = -FVector3f::XAxisVector;
		Vertices[13] = Corners[3]; Normals[13] = -FVector3f::XAxisVector;
		Vertices[14] = Corners[7]; Normals[14] = -FVector3f::XAxisVector;
		Vertices[15] = Corners[4]; Normals[15] = -FVector3f::XAxisVector;
		Tris[6] = FIntVector{ 12, 13, 14 };
		Tris[7] = FIntVector{ 12, 14, 15 };
				
		// +Y face
		Vertices[16] = Corners[3]; Normals[16] = FVector3f::YAxisVector;
		Vertices[17] = Corners[2]; Normals[17] = FVector3f::YAxisVector;
		Vertices[18] = Corners[6]; Normals[18] = FVector3f::YAxisVector;
		Vertices[19] = Corners[7]; Normals[19] = FVector3f::YAxisVector;
		Tris[8] = FIntVector{ 16, 17, 18 };
		Tris[9] = FIntVector{ 16, 18, 19 };

		// -Y face
		Vertices[20] = Corners[1]; Normals[20] = -FVector3f::YAxisVector;
		Vertices[21] = Corners[0]; Normals[21] = -FVector3f::YAxisVector;
		Vertices[22] = Corners[4]; Normals[22] = -FVector3f::YAxisVector;
		Vertices[23] = Corners[5]; Normals[23] = -FVector3f::YAxisVector;
		Tris[10] = FIntVector{ 20, 21, 22 };
		Tris[11] = FIntVector{ 20, 22, 23 };

		TArray<FLinearColor> Colors; Colors.AddUninitialized(Vertices.Num());
		for (int32 VertexIdx = 0; VertexIdx < Colors.Num(); ++VertexIdx)
		{
			Colors[VertexIdx] = FLinearColor::White;
		}

		AddSurface(MoveTemp(Vertices), MoveTemp(Tris), MoveTemp(Normals), MoveTemp(Colors));
	}

	void FRenderingFacade::AddBoxes(const TArray<FBox>& InBoxes)
	{
		for (int32 Idx = 0; Idx < InBoxes.Num(); ++Idx)
		{
			AddBox(InBoxes[Idx]);
		}
	}

	void FRenderingFacade::AddSphere(const FSphere& InSphere, const FLinearColor& InColor)
	{
		AddSphere((FVector3f)InSphere.Center, (float)InSphere.W, InColor);
	}

	void FRenderingFacade::AddSphere(const FVector3f& InCenter, const float InRadius, const FLinearColor& InColor)
	{
		constexpr int32 NumVertices = 26;
		constexpr int32 NumTriangles = 48;

		TArray<FVector3f> Vertices; Vertices.AddUninitialized(NumVertices);
		TArray<FVector3f> VertexNormals; VertexNormals.AddUninitialized(NumVertices);
		TArray<FIntVector> Tris; Tris.AddUninitialized(NumTriangles);

		// Add vertices
		Vertices[0] = FVector3f(0.f, 0.f, 1.f);
		Vertices[1] = FVector3f(0.f, 0.f, -1.f);
		Vertices[2] = FVector3f(0.707107f, 0.f, 0.707107f);
		Vertices[3] = FVector3f(0.5f, -0.5f, 0.707107f);
		Vertices[4] = FVector3f(0.f, -0.707107f, 0.707107f);
		Vertices[5] = FVector3f(-0.5f, -0.5f, 0.707107f);
		Vertices[6] = FVector3f(-0.707107f, 0.f, 0.707107f);
		Vertices[7] = FVector3f(-0.5f, 0.5f, 0.707107f);
		Vertices[8] = FVector3f(0.f, 0.707107f, 0.707107f);
		Vertices[9] = FVector3f(0.5f, 0.5f, 0.707107f);
		Vertices[10] = FVector3f(1.f, 0.f, 0.f);
		Vertices[11] = FVector3f(0.707107f, -0.707107f, 0.f);
		Vertices[12] = FVector3f(0.f, -1.f, 0.f);
		Vertices[13] = FVector3f(-0.707107f, -0.707107f, 0.f);
		Vertices[14] = FVector3f(-1.f, 0.f, 0.f);
		Vertices[15] = FVector3f(-0.707107f, 0.707107f, 0.f);
		Vertices[16] = FVector3f(0.f, 1.f, 0.f);
		Vertices[17] = FVector3f(0.707107f, 0.707107f, 0.f);
		Vertices[18] = FVector3f(0.707107f, 0.f, -0.707107f);
		Vertices[19] = FVector3f(0.5f, -0.5f, -0.707107f);
		Vertices[20] = FVector3f(0.f, -0.707107f, -0.707107f);
		Vertices[21] = FVector3f(-0.5f, -0.5f, -0.707107f);
		Vertices[22] = FVector3f(-0.707107f, 0.f, -0.707107f);
		Vertices[23] = FVector3f(-0.5f, 0.5f, -0.707107f);
		Vertices[24] = FVector3f(0.f, 0.707107f, -0.707107f);
		Vertices[25] = FVector3f(0.5f, 0.5f, -0.707107f);

		for (int32 Idx = 0; Idx < NumVertices; ++Idx)
		{
			VertexNormals[Idx] = Vertices[Idx]; // normalize local vector
			Vertices[Idx] *= InRadius;
			Vertices[Idx] += FVector3f(InCenter);
		}

		// Add triangles
		Tris[0] = FIntVector(0, 2, 3); Tris[1] = FIntVector(0, 3, 4);
		Tris[2] = FIntVector(0, 4, 5); Tris[3] = FIntVector(0, 5, 6);
		Tris[4] = FIntVector(0, 6, 7); Tris[5] = FIntVector(0, 7, 8);
		Tris[6] = FIntVector(0, 8, 9); Tris[7] = FIntVector(0, 9, 2);
		Tris[8] = FIntVector(2, 10, 11); Tris[9] = FIntVector(2, 11, 3);
		Tris[10] = FIntVector(3, 11, 12); Tris[11] = FIntVector(3, 12, 4);
		Tris[12] = FIntVector(4, 12, 13); Tris[13] = FIntVector(4, 13, 5);
		Tris[14] = FIntVector(5, 13, 14); Tris[15] = FIntVector(5, 14, 6);
		Tris[16] = FIntVector(6, 14, 15); Tris[17] = FIntVector(6, 15, 7);
		Tris[18] = FIntVector(7, 15, 16); Tris[19] = FIntVector(7, 16, 8);
		Tris[20] = FIntVector(8, 16, 17); Tris[21] = FIntVector(8, 17, 9);
		Tris[22] = FIntVector(9, 17, 10); Tris[23] = FIntVector(9, 10, 2);
		Tris[24] = FIntVector(10, 18, 19); Tris[25] = FIntVector(10, 19, 11);
		Tris[26] = FIntVector(11, 19, 20); Tris[27] = FIntVector(11, 20, 12);
		Tris[28] = FIntVector(12, 20, 21); Tris[29] = FIntVector(12, 21, 13);
		Tris[30] = FIntVector(13, 21, 22); Tris[31] = FIntVector(13, 22, 14);
		Tris[32] = FIntVector(14, 22, 23); Tris[33] = FIntVector(14, 23, 15);
		Tris[34] = FIntVector(15, 23, 24); Tris[35] = FIntVector(15, 24, 16);
		Tris[36] = FIntVector(16, 24, 25); Tris[37] = FIntVector(16, 25, 17);
		Tris[38] = FIntVector(17, 25, 18); Tris[39] = FIntVector(17, 18, 10);
		Tris[40] = FIntVector(18, 1, 19); Tris[41] = FIntVector(19, 1, 20);
		Tris[42] = FIntVector(20, 1, 21); Tris[43] = FIntVector(21, 1, 22);
		Tris[44] = FIntVector(22, 1, 23); Tris[45] = FIntVector(23, 1, 24);
		Tris[46] = FIntVector(24, 1, 25); Tris[47] = FIntVector(25, 1, 18);

		// Add VertexNormal and VertexColor
		const FLinearColor LinearColor(InColor);
		TArray<FLinearColor> VertexColors; VertexColors.AddUninitialized(Vertices.Num());
		for (int32 VertexIdx = 0; VertexIdx < VertexColors.Num(); ++VertexIdx)
		{
			VertexColors[VertexIdx] = LinearColor;
		}

		AddSurface(MoveTemp(Vertices), MoveTemp(Tris), MoveTemp(VertexNormals), MoveTemp(VertexColors));
	}

	void FRenderingFacade::AddSpheres(const TArray<FSphere>& InSpheres, const FLinearColor& InColor)
	{
		for (int32 Idx = 0; Idx < InSpheres.Num(); ++Idx)
		{
			AddSphere(InSpheres[Idx], InColor);
		}
	}

	void FRenderingFacade::AddTetrahedron(const TArray<FVector3f>& InVertices, const FIntVector4& InIndices)
	{
		constexpr int32 NumVertices = 4;
		constexpr int32 NumTriangles = 4;

		TArray<FVector3f> Vertices; Vertices.AddUninitialized(NumVertices);
		TArray<FIntVector> Tris; Tris.AddUninitialized(NumTriangles);

		Vertices[0] = InVertices[InIndices[0]];
		Vertices[1] = InVertices[InIndices[1]];
		Vertices[2] = InVertices[InIndices[2]];
		Vertices[3] = InVertices[InIndices[3]];

		AddTriangle(Chaos::FTriangle(Vertices[0], Vertices[1], Vertices[2]));
		AddTriangle(Chaos::FTriangle(Vertices[0], Vertices[3], Vertices[1]));
		AddTriangle(Chaos::FTriangle(Vertices[2], Vertices[0], Vertices[3]));
		AddTriangle(Chaos::FTriangle(Vertices[3], Vertices[1], Vertices[2]));
	}

	void FRenderingFacade::AddTetrahedrons(TArray<FVector3f>&& InVertices, TArray<FIntVector4>&& InIndices)
	{
		for (int32 Idx = 0; Idx < InIndices.Num(); ++Idx)
		{
			AddTetrahedron(InVertices, InIndices[Idx]);
		}
	}

	void FRenderingFacade::AddSurfaceBoneWeightsAndIndices(TArray<TArray<float>>&& InBoneWeights, TArray<TArray<int32>>&& InBoneIndices)
	{
		check(!IsConst());
		if (IsValid())
		{
			const int32 GeomIndex = (GeometryNameAttribute.Num() - 1);
			check(GeomIndex >= 0);

			const int32 VertexStart = VertexStartAttribute[GeomIndex];

			if (InBoneWeights.Num() == InBoneIndices.Num())
			{
				const int32 NumVertices = InBoneIndices.Num();

				BoneWeightsAttribute.Add();
				BoneIndicesAttribute.Add();
				TManagedArray<TArray<float>>& BoneWeights = BoneWeightsAttribute.Modify();
				TManagedArray<TArray<int32>>& BoneIndices = BoneIndicesAttribute.Modify();
				for (int32 VtxIndex = 0; VtxIndex < NumVertices; ++VtxIndex)
				{
					const int32 CollectionCVtxIndex = VertexStart + VtxIndex;
					BoneWeights[CollectionCVtxIndex] = MoveTemp(InBoneWeights[VtxIndex]);
					BoneIndices[CollectionCVtxIndex] = MoveTemp(InBoneIndices[VtxIndex]);
				}
			}
		}
		
	}

	void FRenderingFacade::AddSurface(TArray<FVector3f>&& InVertices, TArray<FIntVector>&& InIndices, TArray<FVector3f>&& InNormals, TArray<FLinearColor>&& InColors)
	{
		check(!IsConst());
		if (IsValid())
		{
			auto CollectionVert = [](const Chaos::FVec3& V) { return FVector3f(float(V.X), float(V.Y), float(V.Z)); };

			TManagedArray<FVector3f>& Vertices = VertexAttribute.Modify();
			TManagedArray<FIntVector>& Indices = IndicesAttribute.Modify();

			int32 IndicesStart = IndicesAttribute.AddElements(InIndices.Num());
			int32 VertexStart = VertexAttribute.AddElements(InVertices.Num());
			
			FIntVector * IndiciesDest = Indices.GetData() + IndicesStart;
			FMemory::Memmove((void*)IndiciesDest, InIndices.GetData(), sizeof(FIntVector) * InIndices.Num());

			for (int i = IndicesStart; i < IndicesStart + InIndices.Num(); i++)
			{
				Indices[i][0] += VertexStart;
				Indices[i][1] += VertexStart;
				Indices[i][2] += VertexStart;
			}

			const FVector3f * VerticesDest = Vertices.GetData() + VertexStart;
			FMemory::Memmove((void*)VerticesDest, InVertices.GetData(), sizeof(FVector3f) * InVertices.Num());

			// Add VertexNormals
			TManagedArray<FVector3f>& Normals = VertexNormalAttribute.Modify();

			const FVector3f* VertexNormalsDest = Normals.GetData() + VertexStart;
			FMemory::Memmove((void*)VertexNormalsDest, InNormals.GetData(), sizeof(FVector3f) * InNormals.Num());

			// Add VertexColors
			TManagedArray<FLinearColor>& VertexColors = VertexColorAttribute.Modify();

			const FLinearColor* VertexColorsDest = VertexColors.GetData() + VertexStart;
			FMemory::Memmove((void*)VertexColorsDest, InColors.GetData(), sizeof(FLinearColor) * InColors.Num());
		}
	}

	void FRenderingFacade::AddSurface(TArray<FVector3f>&& InVertices, TArray<FIntVector>&& InIndices, TArray<FVector3f>&& InNormals, TArray<FLinearColor>&& InColors, TArray<TArray<FVector2f>>&& InUVs, TArray<int32>&& InMaterialIDs, TArray<FString>&& InMaterialPaths)
	{
		check(!IsConst());
		if (IsValid())
		{
			const int32 IndicesStart = IndicesAttribute.Num();
			const int32 VertexStart = VertexAttribute.Num();

			AddSurface(MoveTemp(InVertices), MoveTemp(InIndices), MoveTemp(InNormals), MoveTemp(InColors));

			// Add Vertex UVs
			check(InUVs.Num() == InVertices.Num());
			TManagedArray<TArray<FVector2f>>& DestVertexUVs = VertexUVAttribute.Modify();		
			for (int32 VertexID = 0; VertexID < InUVs.Num(); ++VertexID)
			{
				DestVertexUVs[VertexStart + VertexID] = MoveTemp(InUVs[VertexID]);
			}

			// Add Material paths
			const int32 MaterialOffset = MaterialPathAttribute.AddElements(InMaterialPaths.Num());
			for (int32 InMaterialIndex = 0; InMaterialIndex < InMaterialPaths.Num(); ++InMaterialIndex)
			{
				MaterialPathAttribute.Modify()[MaterialOffset + InMaterialIndex] = InMaterialPaths[InMaterialIndex];
			}

			// Add per-Triangle MaterialIDs
			check(InMaterialIDs.Num() == InIndices.Num());
			TManagedArray<int32>& DestTriangleMaterialIDs = MaterialIDAttribute.Modify();
			for (int32 FaceID = 0; FaceID < InMaterialIDs.Num(); ++FaceID)
			{
				DestTriangleMaterialIDs[IndicesStart + FaceID] = InMaterialIDs[FaceID] + MaterialOffset;
			}
		}
	}

	TArray<FRenderingFacade::FTriangleSection> 
	FRenderingFacade::BuildMeshSections(const TArray<FIntVector>& InputIndices, TArray<int32> BaseMeshOriginalIndicesIndex, TArray<FIntVector>& RetIndices) const
	{
		check(!IsConst());
		return FGeometryCollectionSection::BuildMeshSections(ConstCollection, InputIndices, BaseMeshOriginalIndicesIndex, RetIndices);
	}


	int32 FRenderingFacade::StartGeometryGroup(FString InName, const FTransform& InTm)
	{
		check(!IsConst());

		int32 GeomIndex = INDEX_NONE;
		if (IsValid())
		{
			GeomIndex = GeometryNameAttribute.AddElements(1);
			GeometryNameAttribute.Modify()[GeomIndex] = InName;
			GeometryTransformAttribute.Modify()[GeomIndex] = InTm;
			VertexStartAttribute.Modify()[GeomIndex] = VertexAttribute.Num();
			VertexCountAttribute.Modify()[GeomIndex] = 0;
			IndicesStartAttribute.Modify()[GeomIndex] = IndicesAttribute.Num();
			IndicesCountAttribute.Modify()[GeomIndex] = 0;
			GeometrySelectionAttribute.Modify()[GeomIndex] = 0;
			MaterialStartAttribute.Modify()[GeomIndex] = MaterialPathAttribute.Num();
			MaterialCountAttribute.Modify()[GeomIndex] = 0;
		}
		return GeomIndex;
	}

	void FRenderingFacade::EndGeometryGroup(int32 InGeomIndex)
	{
		check(!IsConst());
		if (IsValid())
		{
			check( GeometryNameAttribute.Num()-1 == InGeomIndex );

			if (VertexStartAttribute.Get()[InGeomIndex] < VertexAttribute.Num())
			{
				VertexCountAttribute.Modify()[InGeomIndex] = VertexAttribute.Num() - VertexStartAttribute.Get()[InGeomIndex];

				TManagedArray<int32>& GeomIndexAttr = VertexToGeometryIndexAttribute.Modify();
				for (int i = VertexStartAttribute.Get()[InGeomIndex]; i < VertexAttribute.Num(); i++)
				{
					GeomIndexAttr[i] = InGeomIndex;
				}
			}
			else
			{
				VertexStartAttribute.Modify()[InGeomIndex] = VertexAttribute.Num();
			}

			if (IndicesStartAttribute.Get()[InGeomIndex] < IndicesAttribute.Num())
			{
				IndicesCountAttribute.Modify()[InGeomIndex] = IndicesAttribute.Num() - IndicesStartAttribute.Get()[InGeomIndex];
			}
			else
			{
				IndicesStartAttribute.Modify()[InGeomIndex] = IndicesAttribute.Num();
			}

			if (MaterialStartAttribute.Get()[InGeomIndex] < MaterialPathAttribute.Num())
			{
				MaterialCountAttribute.Modify()[InGeomIndex] = MaterialPathAttribute.Num() - MaterialStartAttribute.Get()[InGeomIndex];
			}
			else
			{
				MaterialStartAttribute.Modify()[InGeomIndex] = MaterialPathAttribute.Num();
			}
		}
	}

	void FRenderingFacade::SetGroupTransform(int32 InGeometryGroupIndex, const FTransform& InTm)
	{
		check(!IsConst());
		if(IsValid())
		{
			// Only support changing the currently open group
			checkf(GeometryNameAttribute.Num() - 1 == InGeometryGroupIndex, TEXT("Attempted to modify a previously closed geometry group transform"));

			GeometryTransformAttribute.Modify()[InGeometryGroupIndex] = InTm;
		}
	}

	FRenderingFacade::FStringIntMap FRenderingFacade::GetGeometryNameToIndexMap() const
	{
		FStringIntMap Map;
		for (int32 i = 0; i < GeometryNameAttribute.Num(); i++)
		{
			Map.Add(GetGeometryName()[i], i);
		}
		return Map;
	}


	int32 FRenderingFacade::NumVerticesOnSelectedGeometry() const
	{
		const TManagedArray<int32>& SelectedGeometry = GeometrySelectionAttribute.Get();
		const TManagedArray<int32>& VertexCount = VertexCountAttribute.Get();
		int32 RetCount = 0;
		for (int i = 0; i < SelectedGeometry.Num(); i++)
			if (SelectedGeometry[i])
				RetCount += VertexCount[i];
		return RetCount;
	}

	void FRenderingFacade::AddCapsule(const float Length, const float Radius, FLinearColor Color, int32 Sides)
	{
		UE::Geometry::FCapsuleGenerator Generator;
		Generator.Radius = FMath::Max(UE_SMALL_NUMBER, Radius);
		Generator.SegmentLength = FMath::Max(UE_SMALL_NUMBER, Length);
		Generator.NumCircleSteps = FMath::Max(3, Sides);
		Generator.Generate();

		TArray<FVector3f> NarrowVerts;
		TArray<FLinearColor> Colors;
		TArray<FIntVector> Triangles;
		NarrowVerts.Reserve(Generator.Vertices.Num());
		Colors.Reserve(Generator.Vertices.Num());
		Triangles.Reserve(Generator.Triangles.Num());

		for(const FVector3d& WideVertex : Generator.Vertices)
		{
			NarrowVerts.Add((FVector3f)WideVertex);
			Colors.Add(Color);
		}

		for(const UE::Geometry::FIndex3i& TriIndices : Generator.Triangles)
		{
			Triangles.Add({TriIndices.A, TriIndices.B, TriIndices.C});
		}

		AddSurface(MoveTemp(NarrowVerts), MoveTemp(Triangles), MoveTemp(Generator.Normals), MoveTemp(Colors));
	}

	void FRenderingFacade::AddFaces(const TArray<FVector3f>& InVertices, TArray<FIntVector>& InIndices, TArray<FLinearColor>& InColors)
	{
		check(!IsConst());
		if(IsValid())
		{
			TArray<FVector3f> ExpandedVerts;
			TArray<FVector3f> ExpandedNormals;
			TArray<FLinearColor> ExpandedColors;
			TArray<FIntVector> ExpandedIndices;

			const int32 NumExpandedVerts = InIndices.Num() * 3;
			ExpandedVerts.SetNumUninitialized(NumExpandedVerts);
			ExpandedNormals.SetNumUninitialized(NumExpandedVerts);
			ExpandedColors.SetNumUninitialized(NumExpandedVerts);
			ExpandedIndices.SetNumUninitialized(InIndices.Num());

			int32 VertexBase = 0;
			for(int32 FaceIndex = 0; FaceIndex < InIndices.Num(); ++FaceIndex)
			{
				const FIntVector Face = InIndices[FaceIndex];
				const FVector3f AB = InVertices[Face[1]] - InVertices[Face[0]];
				const FVector3f AC = InVertices[Face[2]] - InVertices[Face[0]];
				const FVector3f FaceNormal = FVector3f::CrossProduct(AB, AC).GetSafeNormal();

				ExpandedVerts[VertexBase + 0] = InVertices[Face[0]];
				ExpandedVerts[VertexBase + 1] = InVertices[Face[1]];
				ExpandedVerts[VertexBase + 2] = InVertices[Face[2]];

				ExpandedNormals[VertexBase + 0] = FaceNormal;
				ExpandedNormals[VertexBase + 1] = FaceNormal;
				ExpandedNormals[VertexBase + 2] = FaceNormal;

				ExpandedColors[VertexBase + 0] = InColors[Face[0]];
				ExpandedColors[VertexBase + 1] = InColors[Face[1]];
				ExpandedColors[VertexBase + 2] = InColors[Face[2]];

				ExpandedIndices[FaceIndex] = {VertexBase, VertexBase + 1, VertexBase + 2};
				VertexBase += 3;
			}

			AddSurface(MoveTemp(ExpandedVerts), MoveTemp(ExpandedIndices), MoveTemp(ExpandedNormals), MoveTemp(ExpandedColors));
		}
	}

}; // GeometryCollection::Facades


