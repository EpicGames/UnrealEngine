// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GeometryCollection.cpp: FGeometryCollection methods.
=============================================================================*/

#include "GeometryCollection/Facades/CollectionSelectionFacade.h"
#include "GeometryCollection/TransformCollection.h"
#include "GeometryCollection/GeometryCollection.h"

namespace GeometryCollection::Facades
{

	// Groups 
	const FName FSelectionFacade::UnboundGroup = "Unbound";
	const FName FSelectionFacade::WeightedUnboundGroup = "WeightedUnbound";
	const FName FSelectionFacade::BoundGroup = "Bound";
	const FName FSelectionFacade::WeightedBoundGroup = "WeightedBound";

	// Attributes
	const FName FSelectionFacade::IndexAttribute = "Index";
	const FName FSelectionFacade::WeightAttribute = "Weights";
	const FName FSelectionFacade::BoneIndexAttribute = "BoneIndex";

	FSelectionFacade::FSelectionFacade(FManagedArrayCollection& InCollection)
		: ConstCollection(InCollection)
		, Collection(&InCollection) 
	{}

	FSelectionFacade::FSelectionFacade(const FManagedArrayCollection& InCollection)
		: ConstCollection(InCollection)
		, Collection(nullptr)
	{}

	//
	//  Initialization
	//

	void FSelectionFacade::InitUnboundedGroup(FName GroupName, FName DependencyGroup)
	{
		check(!IsConst());

		if (!Collection->HasGroup(GroupName))
		{
			Collection->AddAttribute<TArray<int32>>(FSelectionFacade::IndexAttribute, GroupName, { DependencyGroup });
		}
		ensure(Collection->FindAttributeTyped<TArray<int32>>(FSelectionFacade::IndexAttribute, GroupName) != nullptr);
	}

	void FSelectionFacade::InitWeightedUnboundedGroup(FName GroupName, FName DependencyGroup)
	{
		check(!IsConst());

		if (!Collection->HasGroup(GroupName))
		{
			Collection->AddAttribute<TArray<int32>>(FSelectionFacade::IndexAttribute, GroupName, { DependencyGroup });
			Collection->AddAttribute<TArray<float>>(FSelectionFacade::WeightAttribute, GroupName);
		}
		ensure(Collection->FindAttributeTyped<TArray<int32>>(FSelectionFacade::IndexAttribute, GroupName) != nullptr);
		ensure(Collection->FindAttributeTyped<TArray<float>>(FSelectionFacade::WeightAttribute, GroupName) != nullptr);
	}

	void FSelectionFacade::InitBoundedGroup(FName GroupName, FName DependencyGroup, FName BoneDependencyGroup)
	{
		check(!IsConst());

		if (!Collection->HasGroup(GroupName))
		{
			Collection->AddAttribute<TArray<int32>>(FSelectionFacade::IndexAttribute, GroupName, { DependencyGroup });
			Collection->AddAttribute<int32>(FSelectionFacade::BoneIndexAttribute, GroupName, { BoneDependencyGroup });
		}
		ensure(Collection->FindAttributeTyped<TArray<int32>>(FSelectionFacade::IndexAttribute, GroupName) != nullptr);
		ensure(Collection->FindAttributeTyped<int32>(FSelectionFacade::BoneIndexAttribute, GroupName) != nullptr);
	}

	void FSelectionFacade::InitWeightedBoundedGroup(FName GroupName, FName DependencyGroup, FName BoneDependencyGroup)
	{
		check(!IsConst());

		if (!Collection->HasGroup(GroupName))
		{
			Collection->AddAttribute<TArray<int32>>(FSelectionFacade::IndexAttribute, GroupName, { DependencyGroup });
			Collection->AddAttribute<TArray<float>>(FSelectionFacade::WeightAttribute, GroupName);
			Collection->AddAttribute<int32>(FSelectionFacade::BoneIndexAttribute, GroupName, { BoneDependencyGroup });
		}
		ensure(Collection->FindAttributeTyped<TArray<int32>>(FSelectionFacade::IndexAttribute, GroupName) != nullptr);
		ensure(Collection->FindAttributeTyped<TArray<float>>(FSelectionFacade::WeightAttribute, GroupName) != nullptr);
		ensure(Collection->FindAttributeTyped<int32>(FSelectionFacade::BoneIndexAttribute, GroupName) != nullptr);
	}


	//
	//  AddSelection
	//

	FSelectionFacade::FSelectionKey FSelectionFacade::AddSelection(const TArray<int32>& InIndices, FName DependencyGroup)
	{
		check(!IsConst());

		FName GroupName(FSelectionFacade::UnboundGroup.ToString() + "_" + DependencyGroup.ToString());
		InitUnboundedGroup(GroupName, DependencyGroup);

		int Idx = Collection->AddElements(1, GroupName);
		Collection->ModifyAttribute<TArray<int32>>(FSelectionFacade::IndexAttribute, GroupName)[Idx] = InIndices;
		return FSelectionKey(Idx, FSelectionFacade::UnboundGroup);
	}

	FSelectionFacade::FSelectionKey FSelectionFacade::AddSelection(const TArray<int32>& InIndices, const TArray<float>& InWeights, FName DependencyGroup)
	{
		check(!IsConst());

		FName GroupName(FSelectionFacade::WeightedUnboundGroup.ToString() + "_" + DependencyGroup.ToString());
		InitWeightedUnboundedGroup(GroupName, DependencyGroup);

		int Idx = Collection->AddElements(1, GroupName);
		Collection->ModifyAttribute<TArray<int32>>(FSelectionFacade::IndexAttribute, GroupName)[Idx] = InIndices;
		Collection->ModifyAttribute<TArray<float>>(FSelectionFacade::WeightAttribute, GroupName)[Idx] = InWeights;
		return FSelectionKey(Idx, GroupName);
	}

	FSelectionFacade::FSelectionKey FSelectionFacade::AddSelection(const int32 InBoneIndex, const TArray<int32>& InIndices, FName DependencyGroup, FName BoneDependencyGroup)
	{
		check(!IsConst());

		FName GroupName(FSelectionFacade::BoundGroup.ToString() + "_" + DependencyGroup.ToString());
		InitBoundedGroup(GroupName, DependencyGroup, BoneDependencyGroup);

		int Idx = Collection->AddElements(1, GroupName);
		Collection->ModifyAttribute<TArray<int32>>(FSelectionFacade::IndexAttribute, GroupName)[Idx] = InIndices;
		Collection->ModifyAttribute<int32>(FSelectionFacade::BoneIndexAttribute, GroupName)[Idx] = InBoneIndex;
		return FSelectionKey(Idx, GroupName);
	}

	FSelectionFacade::FSelectionKey FSelectionFacade::AddSelection(const int32 InBoneIndex, const TArray<int32>& InIndices, const TArray<float>& InWeights, FName DependencyGroup, FName BoneDependencyGroup)
	{
		check(!IsConst());

		FName GroupName(FSelectionFacade::WeightedBoundGroup.ToString() + "_" + DependencyGroup.ToString());
		InitWeightedBoundedGroup(GroupName, DependencyGroup, BoneDependencyGroup);

		int Idx = Collection->AddElements(1, GroupName);
		Collection->ModifyAttribute<TArray<int32>>(FSelectionFacade::IndexAttribute, GroupName)[Idx] = InIndices;
		Collection->ModifyAttribute<TArray<float>>(FSelectionFacade::WeightAttribute, GroupName)[Idx] = InWeights;
		Collection->ModifyAttribute<int32>(FSelectionFacade::BoneIndexAttribute, GroupName)[Idx] = InBoneIndex;
		return FSelectionKey(Idx, GroupName);
	}

	//
	//  GetSelection
	//
	void FSelectionFacade::GetSelection(const FSelectionFacade::FSelectionKey& Key, TArray<int32>& OutIndices) const
	{
		if (ConstCollection.HasGroup(Key.GroupName) && 0 <= Key.Index && Key.Index < ConstCollection.NumElements(Key.GroupName))
		{
			if (ConstCollection.FindAttribute<TArray<int32>>(FSelectionFacade::IndexAttribute, Key.GroupName))
				OutIndices = ConstCollection.GetAttribute<TArray<int32>>(FSelectionFacade::IndexAttribute, Key.GroupName)[Key.Index];
		}
	}

	void FSelectionFacade::GetSelection(const FSelectionFacade::FSelectionKey& Key, TArray<int32>& OutIndices, TArray<float>& OutWeights) const
	{
		if (ConstCollection.HasGroup(Key.GroupName) && 0 <= Key.Index && Key.Index < ConstCollection.NumElements(Key.GroupName))
		{
			if (ConstCollection.FindAttribute<TArray<int32>>(FSelectionFacade::IndexAttribute, Key.GroupName))
				OutIndices = ConstCollection.GetAttribute<TArray<int32>>(FSelectionFacade::IndexAttribute, Key.GroupName)[Key.Index];
			if (ConstCollection.FindAttribute<TArray<float>>(FSelectionFacade::WeightAttribute, Key.GroupName))
				OutWeights = ConstCollection.GetAttribute<TArray<float>>(FSelectionFacade::WeightAttribute, Key.GroupName)[Key.Index];
		}
	}

	void FSelectionFacade::GetSelection(const FSelectionFacade::FSelectionKey& Key, int32& OutBoneIndex, TArray<int32>& OutIndices) const
	{
		if (ConstCollection.HasGroup(Key.GroupName) && 0 <= Key.Index && Key.Index < ConstCollection.NumElements(Key.GroupName))
		{
			if (ConstCollection.FindAttribute<TArray<int32>>(FSelectionFacade::IndexAttribute, Key.GroupName))
				OutIndices = ConstCollection.GetAttribute<TArray<int32>>(FSelectionFacade::IndexAttribute, Key.GroupName)[Key.Index];
			if (ConstCollection.FindAttribute<int32>(FSelectionFacade::BoneIndexAttribute, Key.GroupName))
				OutBoneIndex = ConstCollection.GetAttribute<int32>(FSelectionFacade::BoneIndexAttribute, Key.GroupName)[Key.Index];
		}
	}

	void FSelectionFacade::GetSelection(const FSelectionFacade::FSelectionKey& Key, int32& OutBoneIndex, TArray<int32>& OutIndices, TArray<float>& OutWeights) const
	{
		if (ConstCollection.HasGroup(Key.GroupName) && 0 <= Key.Index && Key.Index < ConstCollection.NumElements(Key.GroupName))
		{
			if (ConstCollection.FindAttribute<TArray<int32>>(FSelectionFacade::IndexAttribute, Key.GroupName))
				OutIndices = ConstCollection.GetAttribute<TArray<int32>>(FSelectionFacade::IndexAttribute, Key.GroupName)[Key.Index];
			if (ConstCollection.FindAttribute<TArray<float>>(FSelectionFacade::WeightAttribute, Key.GroupName))
				OutWeights = ConstCollection.GetAttribute<TArray<float>>(FSelectionFacade::WeightAttribute, Key.GroupName)[Key.Index];
			if (ConstCollection.FindAttribute<int32>(FSelectionFacade::BoneIndexAttribute, Key.GroupName))
				OutBoneIndex = ConstCollection.GetAttribute<int32>(FSelectionFacade::BoneIndexAttribute, Key.GroupName)[Key.Index];
		}
	}

	// ------------------------------------------------------------------------------------------------------------------------------------

	void FSelectionFacade::AddSelectionToCollection(const FName InGroupName,
		const TBitArray<>& InSelectionArray,
		const FString InAttribute,
		const bool bStoreAsFloat)
	{
		if (Collection)
		{
			const FName AttrName = FName(*InAttribute);

			if (Collection->NumElements(InGroupName) == InSelectionArray.Num())
			{
				if (!Collection->HasAttribute(AttrName, InGroupName))
				{
					if (bStoreAsFloat)
					{
						TManagedArray<float>& Attr = Collection->AddAttribute<float>(AttrName, InGroupName);

						for (int32 Idx = 0; Idx < InSelectionArray.Num(); ++Idx)
						{
							Attr[Idx] = InSelectionArray[Idx] ? 1.f : 0.f;
						}
					}
					else
					{
						TManagedArray<bool>& Attr = Collection->AddAttribute<bool>(AttrName, InGroupName);

						for (int32 Idx = 0; Idx < InSelectionArray.Num(); ++Idx)
						{
							Attr[Idx] = InSelectionArray[Idx];
						}
					}

					RegisterSelectionInCollection(InGroupName, InAttribute);
				}
			}
		}
	}

	// ------------------------------------------------------------------------------------------------------------------------------------

	void FSelectionFacade::RegisterSelectionInCollection(const FName InGroupName, FString InAttribute)
	{
		static const FName SelectionsGroup = "Selections";
		static const FName SelectionName = "Name";
		static const FName TypeName = "Type";
		
		if (Collection)
		{
			Collection->AddElements(1, SelectionsGroup);

			const FName AttrName = FName(*InAttribute);
		
			if (!Collection->HasAttribute(SelectionName, SelectionsGroup))
			{
				TManagedArray<FString>& NameAttr = Collection->AddAttribute<FString>(SelectionName, SelectionsGroup);
			}
			if (!Collection->HasAttribute(TypeName, SelectionsGroup))
			{
				TManagedArray<FString>& TypeAttr = Collection->AddAttribute<FString>(TypeName, SelectionsGroup);
			}

			TManagedArray<FString>* NameAttr = Collection->FindAttributeTyped<FString>(SelectionName, SelectionsGroup);
			TManagedArray<FString>* TypeAttr = Collection->FindAttributeTyped<FString>(TypeName, SelectionsGroup);

			if (NameAttr && TypeAttr)
			{
				const int32 NumElems = NameAttr->Num();

				(*NameAttr)[NumElems - 1] = InAttribute;
				(*TypeAttr)[NumElems - 1] = InGroupName.ToString();
			}
		}
	}

	// ------------------------------------------------------------------------------------------------------------------------------------

	FSelectionFacade::EErrorCode FSelectionFacade::GetSelectionFromCollection(const FName InGroupName, const FString& InAttribute, TBitArray<>& OutSelectionArray) const
	{
		static const FName SelectionsGroup = "Selections";
		static const FName SelectionName = "Name";
		static const FName TypeName = "Type";

		if (ConstCollection.HasGroup(SelectionsGroup))
		{
			const FName AttrName = FName(*InAttribute);

			const TManagedArray<FString>* NameAttr = ConstCollection.FindAttributeTyped<FString>(SelectionName, SelectionsGroup);
			const TManagedArray<FString>* TypeAttr = ConstCollection.FindAttributeTyped<FString>(TypeName, SelectionsGroup);

			if (NameAttr && TypeAttr && ConstCollection.HasAttribute(AttrName, InGroupName))
			{
				FManagedArrayCollection::EArrayType AttrType = ConstCollection.GetAttributeType(AttrName, InGroupName);

				const TManagedArray<bool>* BoolAttr = nullptr;
				const TManagedArray<float>* FloatAttr = nullptr;

				if (AttrType == FManagedArrayCollection::EArrayType::FBoolType)
				{
					BoolAttr = ConstCollection.FindAttributeTyped<bool>(AttrName, InGroupName);
				}
				else if (AttrType == FManagedArrayCollection::EArrayType::FFloatType)
				{
					FloatAttr = ConstCollection.FindAttributeTyped<float>(AttrName, InGroupName);
				}

				bool bAttrFound = false;
				for (int32 Idx = 0; Idx < NameAttr->Num(); ++Idx)
				{
					if ((*NameAttr)[Idx] == InAttribute && (*TypeAttr)[Idx] == InGroupName.ToString())
					{
						bAttrFound = true;

						if (BoolAttr)
						{
							OutSelectionArray = BoolAttr->GetConstArray();

							return EErrorCode::None;
						}
						else if (FloatAttr)
						{
							const int32 NumElems = FloatAttr->Num();

							OutSelectionArray.Init(false, NumElems);

							for (int32 Idx2 = 0; Idx2 < NumElems; ++Idx2)
							{
								if ((*FloatAttr)[Idx2] > 0.f)
								{
									OutSelectionArray[Idx2] = true;
								}
							}

							return EErrorCode::None;
						}
						else
						{
							return EErrorCode::AttributeCantBeFound;
						}
					}
				}

				if (!bAttrFound)
				{
					return EErrorCode::AttributeCantBeFound;
				}
			}
			else
			{
				return EErrorCode::AttributeCantBeFound;
			}
		}
		else
		{
			return EErrorCode::SelectionGroupCantBeFound;
		}

		return EErrorCode::None;
	}

};
