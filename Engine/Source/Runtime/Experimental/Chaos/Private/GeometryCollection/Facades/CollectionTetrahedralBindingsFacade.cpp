// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GeometryCollection.cpp: FGeometryCollection methods.
=============================================================================*/

#include "GeometryCollection/Facades/CollectionTetrahedralBindingsFacade.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/ManagedArrayAccessor.h"

namespace GeometryCollection::Facades
{
	// [ Duff et al. 2017, "Building an Orthonormal Basis, Revisited" ]
	// Discontinuity at TangentZ.z == 0
	Chaos::PMatrix<float, 3, 3>
	FleshDeformerImpl::GetTangentBasis(const FVector3f& TangentZ)
	{
		const float Sign = copysignf(1.0f, TangentZ.Z);
		const float A = -1.0f / (Sign + TangentZ.Z);
		const float B = TangentZ.Z * TangentZ.Y * A;

		FVector3f TangentX(
			1.0f + Sign * TangentZ.X * TangentZ.X * A,
			Sign * B,
			-Sign * TangentZ.X);
		FVector3f TangentY(
			B,
			Sign + TangentZ.Y * TangentZ.Y * A,
			-TangentZ.Y);

		return Chaos::PMatrix<float, 3, 3>(TangentX, TangentY, TangentZ);
	}

	Chaos::PMatrix<float, 3, 3>
	FleshDeformerImpl::GetOrthogonalBasisVectors(
		const FVector3f& PtA,
		const FVector3f& PtB,
		const FVector3f& PtC)
	{
		FVector3f EdgeBA = PtB - PtA;
		FVector3f EdgeCA = PtC - PtA;
		FVector3f OrthoNorm = FVector3f::CrossProduct(EdgeBA, EdgeCA).GetSafeNormal();
		return GetTangentBasis(OrthoNorm);
	}

	FVector3f
	FleshDeformerImpl::GetRotatedOffsetVector(
		const FVector3f& Offset,
		const FVector3f& RestPtA,
		const FVector3f& RestPtB,
		const FVector3f& RestPtC,
		const FVector3f& CurrPtA,
		const FVector3f& CurrPtB,
		const FVector3f& CurrPtC)
	{
		Chaos::PMatrix<float, 3, 3> BasisVectors = GetOrthogonalBasisVectors(RestPtA, RestPtB, RestPtC);
		Chaos::PMatrix<float, 3, 3> RestRotInv = BasisVectors.Inverse();
		Chaos::PMatrix<float, 3, 3> CurrRot = GetOrthogonalBasisVectors(CurrPtA, CurrPtB, CurrPtC);
		Chaos::PMatrix<float, 3, 3> BasisDelta = RestRotInv * CurrRot;
		return BasisDelta.TransformVector(Offset);
	}

	FVector3f
	FleshDeformerImpl::GetRotatedOffsetVector(
		const FIntVector4& Parents,
		const FVector3f& Offset,
		const TManagedArray<FVector3f>& RestVertices,
		const TArray<Chaos::TVector<Chaos::FRealSingle, 3>>& CurrVertices)
	{
		return GetRotatedOffsetVector(
			Offset,
			RestVertices[Parents[0]], RestVertices[Parents[1]], RestVertices[Parents[2]],
			CurrVertices[Parents[0]], CurrVertices[Parents[1]], CurrVertices[Parents[2]]);
	}
	FString bump;

	FVector3f
	FleshDeformerImpl::GetEmbeddedPosition(
		const int32 SurfaceIndex,
		const TManagedArrayAccessor<FIntVector4>* ParentsArray,
		const TManagedArrayAccessor<FVector4f>* WeightsArray,
		const TManagedArrayAccessor<FVector3f>* OffsetArray,
		const TManagedArray<FVector3f>& RestVertices,
		const TArray<Chaos::TVector<Chaos::FRealSingle, 3>>& CurrVertices)
	{
		const FIntVector4& Parents = ParentsArray->Get()[SurfaceIndex];
		const FVector4f& Weights = WeightsArray->Get()[SurfaceIndex];

		FVector3f Pos = FVector3f::Zero();

		const int32 iEnd = Parents[3] == INDEX_NONE ? 3 : 4;
		for (int32 i = 0; i < iEnd; i++)
		{
			Pos += CurrVertices[Parents[i]] * Weights[i];
		}
		FVector3f Pos2 = Pos;
		FString bumpe = FString::Printf(TEXT("(%f,%f,%f)"), Pos.X, Pos.Y, Pos.Z);
		bump = bumpe;

		// If surface binding, the last index is -1.
		if (Parents[3] == INDEX_NONE)
		{
			const FVector3f& Offset = OffsetArray->Get()[SurfaceIndex];
			const FVector3f RotatedOffset = GetRotatedOffsetVector(Parents, Offset, RestVertices, CurrVertices);
			Pos -= Offset;
		}

		return Pos;
	}


	// Groups 
	const FName FTetrahedralBindings::MeshBindingsGroupName = "MeshBindings";

	// Attributes
	const FName FTetrahedralBindings::MeshIdAttributeName = "MeshId";

	const FName FTetrahedralBindings::ParentsAttributeName = "Parents";
	const FName FTetrahedralBindings::WeightsAttributeName = "Weights";
	const FName FTetrahedralBindings::OffsetsAttributeName = "Offsets";
	const FName FTetrahedralBindings::MaskAttributeName = "Mask";

	// Dependency
	const FName FTetrahedralBindings::TetrahedralGroupDependency = TEXT("Tetrahedral");

	FTetrahedralBindings::FTetrahedralBindings(FManagedArrayCollection& InCollection)
		: MeshIdAttribute(InCollection, MeshIdAttributeName, MeshBindingsGroupName)
	{}

	FTetrahedralBindings::FTetrahedralBindings(const FManagedArrayCollection& InCollection)
		: MeshIdAttribute(InCollection, MeshIdAttributeName, MeshBindingsGroupName)
	{}

	FTetrahedralBindings::~FTetrahedralBindings()
	{}

	void FTetrahedralBindings::DefineSchema()
	{
		check(!IsConst());
		TManagedArray<FString>& MeshIdValues = 
			MeshIdAttribute.IsValid() ? MeshIdAttribute.Modify() : MeshIdAttribute.Add();
	}

	bool FTetrahedralBindings::IsValid() const
	{
		return MeshIdAttribute.IsValid() && 
			(Parents && Parents->IsValid()) && 
			(Weights && Weights->IsValid()) && 
			(Offsets && Offsets->IsValid()) &&
			(Masks && Masks->IsValid());
	}

	FName FTetrahedralBindings::GenerateMeshGroupName(
		const int32 TetMeshIdx,
		const FName& MeshId,
		const int32 LOD)
	{
		const FString MeshIdStr = MeshId.GetPlainNameString();
		FString Str = FString::Printf(TEXT("TetrahedralBindings:TetMeshIdx:%d:%s:%d"), TetMeshIdx, *MeshIdStr, LOD);
		return FName(Str.Len(), *Str);
	}

	bool FTetrahedralBindings::ContainsBindingsGroup(const int32 TetMeshIdx, const FName& MeshId, const int32 LOD) const
	{
		return ContainsBindingsGroup(GenerateMeshGroupName(TetMeshIdx, MeshId, LOD));
	}

	bool FTetrahedralBindings::ContainsBindingsGroup(const FName& GroupName) const
	{
		check(MeshIdAttribute.IsValid());
		const TManagedArray<FString>* MeshIdValues =
			MeshIdAttribute.IsValid() ? MeshIdAttribute.Find() : nullptr;
		return MeshIdValues ? MeshIdValues->Contains(GroupName.ToString()) : false;
	}

	int32 FTetrahedralBindings::GetTetMeshIndex(const FName& MeshId, const int32 LOD) const
	{
		const TManagedArray<FString>* MeshIdValues =
			MeshIdAttribute.IsValid() ? MeshIdAttribute.Find() : nullptr;
		if (MeshIdValues)
		{
			FString Suffix = FString::Printf(TEXT(":%s:%d"), *MeshId.GetPlainNameString(), LOD);
			for (int32 i = 0; i < MeshIdValues->Num(); i++)
			{
				const FString& Entry = (*MeshIdValues)[i];
				if (Entry.EndsWith(Suffix))
				{
					FString Str = Entry;
					Str.RemoveAt(0, FString("TetrahedralBindings:TetMeshIdx:").Len(), EAllowShrinking::No);
					Str.RemoveAt(Str.Len() - Suffix.Len(), Suffix.Len());
					return FCString::Atoi(*Str);
				}
			}
		}
		return INDEX_NONE;
	}

	void FTetrahedralBindings::AddBindingsGroup(const int32 TetMeshIdx, const FName& MeshId, const int32 LOD)
	{
		AddBindingsGroup(GenerateMeshGroupName(TetMeshIdx, MeshId, LOD));
	}

	void FTetrahedralBindings::AddBindingsGroup(const FName& GroupName)
	{
		if (ContainsBindingsGroup(GroupName))
		{
			ReadBindingsGroup(GroupName);
			return;
		}
		check(MeshIdAttribute.IsValid());
		check(MeshIdAttribute.IsPersistent());

		check(!IsConst());
		const int32 Idx = MeshIdAttribute.AddElements(1);
		MeshIdAttribute.Modify()[Idx] = GroupName.ToString();

		Parents.Reset();
		Weights.Reset();
		Offsets.Reset();
		Masks.Reset();
		FManagedArrayCollection& Collection = *MeshIdAttribute.GetCollection();
		Parents.Reset(new TManagedArrayAccessor<FIntVector4>(Collection, ParentsAttributeName, GroupName, TetrahedralGroupDependency));
		Weights.Reset(new TManagedArrayAccessor<FVector4f>(Collection, WeightsAttributeName, GroupName));
		Offsets.Reset(new TManagedArrayAccessor<FVector3f>(Collection, OffsetsAttributeName, GroupName));
		Masks.Reset(new TManagedArrayAccessor<float>(Collection, MaskAttributeName, GroupName));
		Parents->Add(ManageArrayAccessor::EPersistencePolicy::MakePersistent, FGeometryCollection::VerticesGroup);
		Weights->Add();
		Offsets->Add();
		Masks->Add();
	}

	bool FTetrahedralBindings::ReadBindingsGroup(const int32 TetMeshIdx, const FName& MeshId, const int32 LOD)
	{
		return ReadBindingsGroup(GenerateMeshGroupName(TetMeshIdx, MeshId, LOD));
	}

	bool FTetrahedralBindings::ReadBindingsGroup(const FName& GroupName)
	{
		check(MeshIdAttribute.IsValid());
		Parents.Reset();
		Weights.Reset();
		Offsets.Reset();
		Masks.Reset();
		if (!MeshIdAttribute.Find()->Contains(GroupName.ToString()))
		{
			return false;
		}
		// This is an existing group, so find the existing bindings arrays.
		if (!IsConst())
		{
			FManagedArrayCollection* Collection = MeshIdAttribute.GetCollection();
			Parents.Reset(new TManagedArrayAccessor<FIntVector4>(*Collection, ParentsAttributeName, GroupName, TetrahedralGroupDependency));
			Weights.Reset(new TManagedArrayAccessor<FVector4f>(*Collection, WeightsAttributeName, GroupName));
			Offsets.Reset(new TManagedArrayAccessor<FVector3f>(*Collection, OffsetsAttributeName, GroupName));
			Masks.Reset(new TManagedArrayAccessor<float>(*Collection, MaskAttributeName, GroupName));
		}
		else
		{
			const FManagedArrayCollection& ConstCollection = MeshIdAttribute.GetConstCollection();
			Parents.Reset(new TManagedArrayAccessor<FIntVector4>(ConstCollection, ParentsAttributeName, GroupName, TetrahedralGroupDependency));
			Weights.Reset(new TManagedArrayAccessor<FVector4f>(ConstCollection, WeightsAttributeName, GroupName));
			Offsets.Reset(new TManagedArrayAccessor<FVector3f>(ConstCollection, OffsetsAttributeName, GroupName));
			Masks.Reset(new TManagedArrayAccessor<float>(ConstCollection, MaskAttributeName, GroupName));
		}
		return Parents->IsValid() && Weights->IsValid() && Offsets->IsValid() && Masks->IsValid();
	}

	void FTetrahedralBindings::RemoveBindingsGroup(const int32 TetMeshIdx, const FName& MeshId, const int32 LOD)
	{
		RemoveBindingsGroup(GenerateMeshGroupName(TetMeshIdx, MeshId, LOD));
	}

	void FTetrahedralBindings::RemoveBindingsGroup(const FName& GroupName)
	{
		check(!IsConst());
		TManagedArray<FString>& MeshIdValues = MeshIdAttribute.Modify();
		int32 Idx = MeshIdValues.Find(GroupName.ToString());
		if (Idx != INDEX_NONE)
		{
			TArray<int32> Indices;
			Indices.Add(Idx);
			MeshIdValues.RemoveElements(Indices);
		}

		FManagedArrayCollection& Collection = *MeshIdAttribute.GetCollection();
		if (Parents)
		{
			Parents->Remove();
			Parents.Reset();
		}
		if (Weights)
		{
			Weights->Remove();
			Weights.Reset();
		}
		if (Offsets)
		{
			Offsets->Remove();
			Offsets.Reset();
		}
		if (Masks)
		{
			Masks->Remove();
			Masks.Reset();
		}
		// Only drop the group if it's empty at this point?
		if (Collection.NumAttributes(GroupName) == 0)
		{
			Collection.RemoveGroup(GroupName);
		}
	}

	void 
	FTetrahedralBindings::SetBindingsData(
		const TArray<FIntVector4>& ParentsIn,
		const TArray<FVector4f>& WeightsIn,
		const TArray<FVector3f>& OffsetsIn,
		const TArray<float>& MaskIn)
	{
		check(!IsConst());
		check(IsValid());
		check((ParentsIn.Num() == WeightsIn.Num()) && (ParentsIn.Num() == OffsetsIn.Num()) && (ParentsIn.Num() == MaskIn.Num()));

		const int32 Num = ParentsIn.Num();
		const int32 CurrNum = Parents->Num();//Collection.NumElements(CurrGroupName);
		Parents->AddElements(Num - CurrNum); // Resizes the group
		TManagedArray<FIntVector4>& ParentsValues = Parents->Modify();
		TManagedArray<FVector4f>& WeightsValues = Weights->Modify();
		TManagedArray<FVector3f>& OffsetsValues = Offsets->Modify();
		TManagedArray<float>& MaskValues = Masks->Modify();
		for (int32 i = 0; i < Num; i++)
		{
			ParentsValues[i] = ParentsIn[i];
			WeightsValues[i] = WeightsIn[i];
			OffsetsValues[i] = OffsetsIn[i];
			MaskValues[i] = MaskIn[i];
		}
	}


};


