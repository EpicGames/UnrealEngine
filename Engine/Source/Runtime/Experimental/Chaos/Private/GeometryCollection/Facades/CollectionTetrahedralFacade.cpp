// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GeometryCollection.cpp: FGeometryCollection methods.
=============================================================================*/

#include "GeometryCollection/Facades/CollectionTetrahedralFacade.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/ManagedArrayAccessor.h"
#include "Chaos/Tetrahedron.h"
#include "Chaos/BoundingVolumeHierarchy.h"

namespace GeometryCollection::Facades
{
	FTetrahedralFacade::FTetrahedralFacade(FManagedArrayCollection& InCollection)
		: ConstCollection(InCollection)
		, Collection(&InCollection)
		, Tetrahedron(InCollection, "Tetrahedron", "Tetrahedral")
		, TetrahedronStart(InCollection, "TetrahedronStart", FGeometryCollection::GeometryGroup)
		, TetrahedronCount(InCollection, "TetrahedronCount", FGeometryCollection::GeometryGroup)
		, VertexStart(InCollection, "VertexStart", FGeometryCollection::GeometryGroup)
		, VertexCount(InCollection, "VertexCount", FGeometryCollection::GeometryGroup)
		, Vertex(InCollection, "Vertex", FGeometryCollection::VerticesGroup)
	{}

	FTetrahedralFacade::FTetrahedralFacade(const FManagedArrayCollection& InCollection)
		: ConstCollection(InCollection)
		, Collection(nullptr)
		, Tetrahedron(InCollection, "Tetrahedron", "Tetrahedral")
		, TetrahedronStart(InCollection, "TetrahedronStart", FGeometryCollection::GeometryGroup)
		, TetrahedronCount(InCollection, "TetrahedronCount", FGeometryCollection::GeometryGroup)
		, VertexStart(InCollection, "VertexStart", FGeometryCollection::GeometryGroup)
		, VertexCount(InCollection, "VertexCount", FGeometryCollection::GeometryGroup)
		, Vertex(InCollection, "Vertex", FGeometryCollection::VerticesGroup)
	{}

	FTetrahedralFacade::~FTetrahedralFacade()
	{}

	void FTetrahedralFacade::DefineSchema()
	{
	}

	bool FTetrahedralFacade::IsValid() const
	{
		return Tetrahedron.IsValid() && TetrahedronStart.IsValid() && TetrahedronCount.IsValid()
			&& VertexStart.IsValid() && VertexCount.IsValid() && Vertex.IsValid();
	}

	bool FTetrahedralFacade::Intersection(
		const TConstArrayView<Chaos::Softs::FSolverVec3>& SamplePositions,
		const TConstArrayView<Chaos::Softs::FSolverVec3>& TetarhedronPositions,
		TArray<TetrahedralParticleEmbedding>& OutIntersections) const
	{
		using namespace Chaos;

		auto TetrahedralMeshIndices = [&]() {
			TArray<int32> RetList;
			for (int32 i = 0; i < TetrahedronCount.Num(); i++) if (TetrahedronCount[i] > 0) RetList.Add(i);
			return RetList;
		};

		for (int32 MeshIndex : TetrahedralMeshIndices())
		{
			int32 TetStart = TetrahedronStart[MeshIndex];
			int32 TetCount = TetrahedronCount[MeshIndex];
			int32 VrtStart = VertexStart[MeshIndex];
			// Build Tetrahedra
			typedef Chaos::TTetrahedron<Chaos::FReal> FTet;
			TArray<FTet> Tets;
			TArray<FTet*> TetPtrs;

			Tets.SetNumUninitialized(TetCount);
			TetPtrs.SetNumUninitialized(TetCount);

			for (int32 Tdx = 0; Tdx < TetCount; Tdx++)
			{
				const int32 Idx = TetStart + Tdx;
				//Test shows offset VrtStart is not needed:
				//FIntVector4 Tet = Tetrahedron[Idx] - FIntVector4(VrtStart);
				FIntVector4 Tet = Tetrahedron[Idx];
				Tets[Tdx] = FTet(TetarhedronPositions[Tet[0]], TetarhedronPositions[Tet[1]], TetarhedronPositions[Tet[2]], TetarhedronPositions[Tet[3]]);
				TetPtrs[Tdx] = &Tets[Tdx];
			}

			Chaos::TBoundingVolumeHierarchy< TArray<FTet*>, TArray<int32>, Chaos::FReal, 3> BVH(TetPtrs);
			for (int Pdx = 0; Pdx < SamplePositions.Num(); Pdx++)
			{
				const auto& Pos = SamplePositions[Pdx];
				TArray<int32> Intersections = BVH.FindAllIntersections(Pos);
				for (int32 j = 0; j < Intersections.Num(); j++)
				{
					const int32 TetIdx = Intersections[j];
					if (!Tets[TetIdx].Outside(Pos, 0))
					{
						TVector<FReal, 4> W = Tets[TetIdx].GetBarycentricCoordinates(Pos);
						OutIntersections.Add({ Pdx, MeshIndex, TetIdx, { (float)W[0],(float)W[1],(float)W[2],(float)W[3] } });
						break;
					}
				}
			}
		}
		return OutIntersections.Num()>0;
	}
};


