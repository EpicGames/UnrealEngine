// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/Facades/CollectionTetrahedralMetricsFacade.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/ManagedArrayAccessor.h"

namespace GeometryCollection::Facades
{
	// Groups

	// Attributes
	const FName FTetrahedralMetrics::SignedVolumeAttributeName = "SignedVolume";
	const FName FTetrahedralMetrics::AspectRatioAttributeName = "AspectRatio";

	FTetrahedralMetrics::FTetrahedralMetrics(FManagedArrayCollection& InCollection)
		: SignedVolumeAttribute(InCollection, SignedVolumeAttributeName, "Tetrahedral")
		, AspectRatioAttribute(InCollection, AspectRatioAttributeName, "Tetrahedral")
	{
		DefineSchema();
	}

	FTetrahedralMetrics::FTetrahedralMetrics(const FManagedArrayCollection& InCollection)
		: SignedVolumeAttribute(InCollection, SignedVolumeAttributeName, "Tetrahedral")
		, AspectRatioAttribute(InCollection, AspectRatioAttributeName, "Tetrahedral")
	{}

	FTetrahedralMetrics::~FTetrahedralMetrics()
	{}

	void FTetrahedralMetrics::DefineSchema()
	{
		check(!IsConst());
		SignedVolumeAttribute.Add();
		AspectRatioAttribute.Add();
	}

	bool FTetrahedralMetrics::IsValid() const
	{
		return SignedVolumeAttribute.IsValid() && AspectRatioAttribute.IsValid();
	}

} // namesapce GeometryCollection::Facades
