// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GeometryCollection.cpp: FGeometryCollection methods.
=============================================================================*/

#include "GeometryCollection/Facades/CollectionTetrahedralSkeletalBindingsFacade.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/ManagedArrayAccessor.h"


namespace GeometryCollection::Facades
{
	// Groups 
	const FName FTetrahedralSkeletalBindings::MeshBindingsGroupName = "SkeletalBindings";
	const FName FTetrahedralSkeletalBindings::MeshBindingsIdGroupName = "SkeletalBindingsId";

	// Attributes
	const FName FTetrahedralSkeletalBindings::MeshIdAttributeName = "MeshId";
	const FName FTetrahedralSkeletalBindings::MeshIndexAttributeName = "MeshIdIndex";
	const FName FTetrahedralSkeletalBindings::TetrahedronIndexAttributeName = "TetrahedronIndex";
	const FName FTetrahedralSkeletalBindings::WeightsAttributeName = "Weights";
	const FName FTetrahedralSkeletalBindings::SkeletalIndexAttributeName = "SkeletalIndex";

	FTetrahedralSkeletalBindings::FTetrahedralSkeletalBindings(FManagedArrayCollection& InCollection)
		: MeshIdAttribute(InCollection, MeshIdAttributeName, MeshBindingsIdGroupName)
		, MeshGroupIndexAttribute(InCollection, MeshIndexAttributeName, MeshBindingsGroupName, MeshBindingsIdGroupName)
		, TetrahedronIndexAttribute(InCollection, TetrahedronIndexAttributeName, MeshBindingsGroupName, "Tetrahedral")
		, WeightsAttribute(InCollection, WeightsAttributeName, MeshBindingsGroupName)
		, SkeletalIndexAttribute(InCollection, SkeletalIndexAttributeName, MeshBindingsGroupName)
		, TetrahedronAttribute(InCollection, "Tetrahedron", "Tetrahedral")
		, VerticesAttribute(InCollection, "Vertex", FGeometryCollection::VerticesGroup)
	{
		DefineSchema();
	}

	FTetrahedralSkeletalBindings::FTetrahedralSkeletalBindings(const FManagedArrayCollection& InCollection)
		: MeshIdAttribute(InCollection, MeshIdAttributeName, MeshBindingsIdGroupName)
		, MeshGroupIndexAttribute(InCollection, MeshIndexAttributeName, MeshBindingsGroupName, MeshBindingsIdGroupName)
		, TetrahedronIndexAttribute(InCollection, TetrahedronIndexAttributeName, MeshBindingsGroupName, "Tetrahedral")
		, WeightsAttribute(InCollection, WeightsAttributeName, MeshBindingsGroupName)
		, SkeletalIndexAttribute(InCollection, SkeletalIndexAttributeName, MeshBindingsGroupName) 
		, TetrahedronAttribute(InCollection, "Tetrahedron", "Tetrahedral")
		, VerticesAttribute(InCollection, "Vertex", FGeometryCollection::VerticesGroup)
	{}

	FTetrahedralSkeletalBindings::~FTetrahedralSkeletalBindings()
	{}

	void FTetrahedralSkeletalBindings::DefineSchema()
	{
		check(!IsConst());
		MeshIdAttribute.Add();
		MeshGroupIndexAttribute.Add();
		TetrahedronIndexAttribute.Add();
		WeightsAttribute.Add();
		SkeletalIndexAttribute.Add();
	}

	bool FTetrahedralSkeletalBindings::IsValid() const
	{
		return MeshIdAttribute.IsValid() && MeshGroupIndexAttribute.IsValid() && TetrahedronIndexAttribute.IsValid()
			&& WeightsAttribute.IsValid() && SkeletalIndexAttribute.IsValid() && TetrahedronAttribute.IsValid()
			&& VerticesAttribute.IsValid();
	}

	FString FTetrahedralSkeletalBindings::GenerateMeshGroupName( const int32 TetMeshIdx,const FName& MeshId,const int32 LOD)
	{
		return FString::Printf(TEXT("v1:%d:%s:%d"), TetMeshIdx, *MeshId.ToString(), LOD);
	}

	void FTetrahedralSkeletalBindings::SetBindings(const FString& InMeshGroupIndex, const TArray<int32>& InTetrahedronIndex,
												   const TArray<FVector4f>& WeightsIn, const TArray<int32>& InSkeletalIndex)
	{
		check(!IsConst());
		if (InTetrahedronIndex.Num() && InTetrahedronIndex.Num() == WeightsIn.Num() && InTetrahedronIndex.Num() == InSkeletalIndex.Num())
		{
			TManagedArray<FString>& MeshId = MeshIdAttribute.Modify();
			TManagedArray<int32>& MeshGroupIndex = MeshGroupIndexAttribute.Modify();
			TManagedArray<int32>& TetrahedronIndex = TetrahedronIndexAttribute.Modify();
			TManagedArray<FVector4f>& Weights = WeightsAttribute.Modify();
			TManagedArray<int32>& SkeletalIndex = SkeletalIndexAttribute.Modify();

			int32 GroupIndex = MeshId.Find(InMeshGroupIndex);
			if (GroupIndex == INDEX_NONE)
			{
				GroupIndex = MeshIdAttribute.AddElements(1);
			}
			MeshId[GroupIndex] = InMeshGroupIndex;

			int32 NumTetrahedron = InTetrahedronIndex.Num();
			int32 StartIndex = MeshGroupIndexAttribute.AddElements(NumTetrahedron);
			for (int32 CurrentIndex = StartIndex, i = 0; CurrentIndex < StartIndex + NumTetrahedron; CurrentIndex++, i++)
			{
				MeshGroupIndex[CurrentIndex] = GroupIndex;
				TetrahedronIndex[CurrentIndex] = InTetrahedronIndex[i];
				Weights[CurrentIndex] = WeightsIn[i];
				SkeletalIndex[CurrentIndex] = InSkeletalIndex[i];
			}
		}
	}


	bool FTetrahedralSkeletalBindings::CalculateBindings(const FString & InKey, const TArray<FVector3f>& InVertices, TArray<FVector>& OutPosition, TArray<bool>* OutInfluence) const
	{
		auto UEVert3d = [](FVector3f V) { return FVector3d(V.X, V.Y, V.Z); };

		if (!IsValid())
		{
			return false;
		}
		if(InVertices.Num())
		{
			int32 GroupIndex = MeshIdAttribute.Get().Find(InKey);
			if (GroupIndex == INDEX_NONE)
			{
				return false;
			}

			const TManagedArray<int32>& MeshGroupIndex = MeshGroupIndexAttribute.Get();
			const TManagedArray<int32>& TetrahedronIndex = TetrahedronIndexAttribute.Get();
			const TManagedArray<FVector4f>& Weights = WeightsAttribute.Get();
			const TManagedArray<int32>& SkeletalIndices = SkeletalIndexAttribute.Get();
			const TManagedArray<FIntVector4>& Tetrahedron = TetrahedronAttribute.Get();

			int32 NumBindings = MeshGroupIndex.Num();
			for (int32 BindingIndex = 0; BindingIndex < MeshGroupIndex.Num(); BindingIndex++)
			{
				if (MeshGroupIndex[BindingIndex] == GroupIndex)
				{
					int32 TetIndex = TetrahedronIndex[BindingIndex];
					if (0 <= TetIndex && TetIndex < TetrahedronAttribute.Num())
					{
						const FIntVector4& Tet = Tetrahedron[TetIndex];
						int32 SkeletonIndex = SkeletalIndices[BindingIndex];
						if (0 <= SkeletonIndex && SkeletonIndex < OutPosition.Num())
						{
							const FVector4 Weight(Weights[BindingIndex][0], Weights[BindingIndex][1], Weights[BindingIndex][2], Weights[BindingIndex][3]);
							FVector X0 = InVertices.IsValidIndex(Tet[0]) ? UEVert3d(InVertices[Tet[0]]) : FVector();
							FVector X1 = InVertices.IsValidIndex(Tet[0]) ? UEVert3d(InVertices[Tet[1]]) : FVector();
							FVector X2 = InVertices.IsValidIndex(Tet[0]) ? UEVert3d(InVertices[Tet[2]]) : FVector();
							FVector X3 = InVertices.IsValidIndex(Tet[0]) ? UEVert3d(InVertices[Tet[3]]) : FVector();
							OutPosition[SkeletonIndex] = Weight[0] * X0 + Weight[1] * X1 + Weight[2] * X2 + Weight[3] * X3;
							if (OutInfluence != nullptr) if (SkeletonIndex < OutInfluence->Num()) (*OutInfluence)[SkeletonIndex] = true;
						}
					}
				}
			}
		}
		return true;
	}
};


