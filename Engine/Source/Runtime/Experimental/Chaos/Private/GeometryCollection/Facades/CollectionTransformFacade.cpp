// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/Facades/CollectionTransformFacade.h"
#include "GeometryCollection/TransformCollection.h"
#include "GeometryCollection/Facades/CollectionHierarchyFacade.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"

namespace GeometryCollection::Facades
{
	FCollectionTransformFacade::FCollectionTransformFacade(FManagedArrayCollection& InCollection)
		: ConstCollection(InCollection)
		, Collection(&InCollection)
		, ParentAttribute(InCollection, FTransformCollection::ParentAttribute, FTransformCollection::TransformGroup)
		, ChildrenAttribute(InCollection, FTransformCollection::ChildrenAttribute, FTransformCollection::TransformGroup)
		, TransformAttribute(InCollection, FTransformCollection::TransformAttribute, FTransformCollection::TransformGroup)
		, BoneNameAttribute(InCollection, "BoneName", FTransformCollection::TransformGroup)
	{}

	FCollectionTransformFacade::FCollectionTransformFacade(const FManagedArrayCollection& InCollection)
		: ConstCollection(InCollection)
		, Collection(nullptr)
		, ParentAttribute(InCollection, FTransformCollection::ParentAttribute, FTransformCollection::TransformGroup)
		, ChildrenAttribute(InCollection, FTransformCollection::ChildrenAttribute, FTransformCollection::TransformGroup)
		, TransformAttribute(InCollection, FTransformCollection::TransformAttribute, FTransformCollection::TransformGroup)
		, BoneNameAttribute(InCollection, "BoneName", FTransformCollection::TransformGroup)
	{}

	bool FCollectionTransformFacade::IsValid() const
	{
		return ParentAttribute.IsValid() && ChildrenAttribute.IsValid() && TransformAttribute.IsValid();
	}

	int32 FCollectionTransformFacade::Num() const
	{
		return ParentAttribute.Num();
	}

	bool FCollectionTransformFacade::HasBoneNameAttribute() const
	{
		return BoneNameAttribute.IsValid();
	}

	FString FCollectionTransformFacade::BoneName(int32 Index) const
	{
		if(BoneNameAttribute.IsValid() && BoneNameAttribute.IsValidIndex(Index))
		{
			return BoneNameAttribute[Index];
		}
		return FString();
	}

	TMap<FString, int32> FCollectionTransformFacade::BoneNameIndexMap() const
	{
		TMap<FString, int32> BoneNameToIndex;
		if (BoneNameAttribute.IsValid())
		{
			for (int32 BoneIdx = 0; BoneIdx < BoneNameAttribute.Num(); ++BoneIdx)
			{
				BoneNameToIndex.Add(BoneNameAttribute[BoneIdx], BoneIdx);
			}
		}
		return MoveTemp(BoneNameToIndex);
	}

	TArray<int32> FCollectionTransformFacade::GetRootIndices() const
	{
		return Chaos::Facades::FCollectionHierarchyFacade::GetRootIndices(ParentAttribute);
	}

	FTransform FCollectionTransformFacade::GetRootTransform() const
	{
		const TArray<int32> RootIndices = GetRootIndices();
		if (RootIndices.Num() > 0)
		{
			const int32 RootIndex = RootIndices[0];
			if (TransformAttribute.IsValid())
			{
				return FTransform(TransformAttribute[RootIndex]);
			}
		}
		return FTransform::Identity;
	}

	TArray<FTransform> FCollectionTransformFacade::ComputeCollectionSpaceTransforms() const
	{
		TArray<FTransform> OutTransforms;

		if(IsValid())
		{
			const TManagedArray<FTransform3f>& BoneTransforms = TransformAttribute.Get();
			const TManagedArray<int32>& Parents = ParentAttribute.Get();

			GeometryCollectionAlgo::GlobalMatrices(BoneTransforms, Parents, OutTransforms);
		}

		return OutTransforms;
	}

	FTransform FCollectionTransformFacade::ComputeCollectionSpaceTransform(int32 BoneIdx) const
	{
		const TManagedArray<FTransform3f>& BoneTransforms = TransformAttribute.Get();
		const TManagedArray<int32>& Parents = ParentAttribute.Get();

		return GeometryCollectionAlgo::GlobalMatrix(BoneTransforms, Parents, BoneIdx);
	}

	void FCollectionTransformFacade::SetPivot(const FTransform& InTransform)
	{
		Transform(InTransform.Inverse());
	}

	void FCollectionTransformFacade::Transform(const FTransform& InTransform)
	{
		// Update only root transforms
		const TArray<int32>& RootIndices = GetRootIndices();
		if (TransformAttribute.IsValid())
		{
			TManagedArray<FTransform3f>& Transforms = TransformAttribute.Modify();

			for (int32 Idx : RootIndices)
			{
				Transforms[Idx] = Transforms[Idx] * FTransform3f(InTransform);
			}
		}
	}

	void FCollectionTransformFacade::Transform(const FTransform& InTransform, const TArray<int32>& InSelection)
	{
		if (TransformAttribute.IsValid())
		{
			TManagedArray<FTransform3f>& Transforms = TransformAttribute.Modify();

			for (int32 Idx : InSelection)
			{
				if (Transforms.IsValidIndex(Idx))
				{
					Transforms[Idx] = Transforms[Idx] * FTransform3f(InTransform);
				}
			}
		}
	}

	/**
	* Builds an FMatrix from FVector4 column vectors
	*/
	static FMatrix SetMatrix(const FVector4& Column0, const FVector4& Column1, const FVector4& Column2, const FVector4& Column3)
	{
		//
		// Matrix elements are accessed with M[RowIndex][ColumnumnIndex]
		//
		FMatrix Matrix;

		Matrix.M[0][0] = Column0[0]; Matrix.M[1][0] = Column0[1]; Matrix.M[2][0] = Column0[2]; Matrix.M[3][0] = Column0[3];
		Matrix.M[0][1] = Column1[0]; Matrix.M[1][1] = Column1[1]; Matrix.M[2][1] = Column1[2]; Matrix.M[3][1] = Column1[3];
		Matrix.M[0][2] = Column2[0]; Matrix.M[1][2] = Column2[1]; Matrix.M[2][2] = Column2[2]; Matrix.M[3][2] = Column2[3];
		Matrix.M[0][3] = Column3[0]; Matrix.M[1][3] = Column3[1]; Matrix.M[2][3] = Column3[2]; Matrix.M[3][3] = Column3[3];

		return Matrix;
	}

	FMatrix FCollectionTransformFacade::BuildMatrix(const FVector& Translate,
		const uint8 RotationOrder,
		const FVector& Rotate,
		const FVector& InScale,
		const FVector& Shear,
		const float UniformScale,
		const FVector& RotatePivot,
		const FVector& ScalePivot,
		const bool InvertTransformation)
	{
// 
//   M = [Sp-1]x[S]x[Sh]x[Sp]x[St]x[Rp-1]x[Ro]x[R]x[Rp]x[Rt]x[T]
//
					
		// Scale Pivot point
		FMatrix Sp = SetMatrix(FVector4(1.f, 0.f, 0.f, ScalePivot.X), FVector4(0.f, 1.f, 0.f, ScalePivot.Y), FVector4(0.f, 0.f, 1.f, ScalePivot.Z), FVector4(0.f, 0.f, 0.f, 1.f));
		FMatrix SpInv = Sp.Inverse();

		// Scale
		FVector Scale = InScale * UniformScale;
		FMatrix S = SetMatrix(FVector4(Scale.X, 0.f, 0.f, 0.f), FVector4(0.f, Scale.Y, 0.f, 0.f), FVector4(0.f, 0.f, Scale.Z, 0.f), FVector4(0.f, 0.f, 0.f, 1.f));

		// Shear
		FMatrix Sh = SetMatrix(FVector4(1.f, Shear.X, Shear.Y, 0.f), FVector4(0.f, 1.f, Shear.Z, 0.f), FVector4(0.f, 0.f, 1.f, 0.f), FVector4(0.f, 0.f, 0.f, 1.f));
	
		// Scale Pivot translation
		FMatrix St = FMatrix::Identity;

		// Rotate Pivot point
		FMatrix Rp = SetMatrix(FVector4(1.f, 0.f, 0.f, RotatePivot.X), FVector4(0.f, 1.f, 0.f, RotatePivot.Y), FVector4(0.f, 0.f, 1.f, RotatePivot.Z), FVector4(0.f, 0.f, 0.f, 1.f));
		FMatrix RpInv = Rp.Inverse();

		// Rotation Orientation
		FMatrix Ro = FMatrix::Identity;

		// Rotation
		double Sx = sin(FMath::DegreesToRadians(Rotate.X)); double Sy = sin(FMath::DegreesToRadians(Rotate.Y)); double Sz = sin(FMath::DegreesToRadians(Rotate.Z));
		double Cx = cos(FMath::DegreesToRadians(Rotate.X)); double Cy = cos(FMath::DegreesToRadians(Rotate.Y)); double Cz = cos(FMath::DegreesToRadians(Rotate.Z));

		FMatrix Rx = SetMatrix(FVector4(1.f, 0.f, 0.f, 0.f), FVector4(0.f, Cx, -1.f * Sx, 0.f), FVector4(0.f, Sx, Cx, 0.f), FVector4(0.f, 0.f, 0.f, 1.f));
		FMatrix Ry = SetMatrix(FVector4(Cy, 0.f, Sy, 0.f), FVector4(0.f, 1.f, 0.f, 0.f), FVector4(-1.f * Sy, 0.f, Cy, 0.f), FVector4(0.f, 0.f, 0.f, 1.f));
		FMatrix Rz = SetMatrix(FVector4(Cz, -1.f * Sz, 0.f, 0.f), FVector4(Sz, Cz, 0.f, 0.f), FVector4(0.f, 0.f, 1.f, 0.f), FVector4(0.f, 0.f, 0.f, 1.f));

		FMatrix R;
		if (RotationOrder == 0) // XYZ
		{
			R = Rx * Ry; R = R * Rz;
		}
		else if (RotationOrder == 1) // YZX
		{
			R = Ry * Rz; R = R * Rx;
		}
		else if (RotationOrder == 2) // ZXY
		{
			R = Rz * Rx; R = R * Ry;
		}
		else if (RotationOrder == 3) // XZY
		{
			R = Rx * Rz; R = R * Ry;
		}
		else if (RotationOrder == 4) // YXZ
		{
			R = Ry * Rx; R = R * Rz;
		}
		else if (RotationOrder == 5) // ZYX
		{
			R = Rz * Ry; R = R * Rx;
		}

		// Rotate pivot translation
		FMatrix Rt = FMatrix::Identity;

		// Translate
		FMatrix T = SetMatrix(FVector4(1.f, 0.f, 0.f, Translate.X), FVector4(0.f, 1.f, 0.f, Translate.Y), FVector4(0.f, 0.f, 1.f, Translate.Z), FVector4(0.f, 0.f, 0.f, 1.f));

		FMatrix MatrixResult = SpInv * S;
		MatrixResult = MatrixResult * Sh;
		MatrixResult = MatrixResult * Sp;
		MatrixResult = MatrixResult * St;

		MatrixResult = MatrixResult * RpInv;
		MatrixResult = MatrixResult * Ro;
		MatrixResult = MatrixResult * R;
		MatrixResult = MatrixResult * Rp;
		MatrixResult = MatrixResult * Rt;

		MatrixResult = MatrixResult * T;

		if (InvertTransformation)
		{
			MatrixResult = MatrixResult.Inverse();
		}

		return MatrixResult;
	}


	FTransform FCollectionTransformFacade::BuildTransform(const FVector& Translate,
		const uint8 RotationOrder,
		const FVector& Rotate,
		const FVector& InScale,
		const float UniformScale,
		const FVector& RotatePivot,
		const FVector& ScalePivot,
		const bool InvertTransformation)
	{
		return FTransform(BuildMatrix(Translate,
			RotationOrder,
			Rotate,
			InScale,
			FVector(0.0),
			UniformScale,
			RotatePivot,
			ScalePivot,
			InvertTransformation));
	}


	void FCollectionTransformFacade::SetBoneTransformToIdentity(int32 BoneIdx)
	{
		if (TransformAttribute.IsValid())
		{
			TManagedArray<FTransform3f>& BoneTransforms = TransformAttribute.Modify();
			BoneTransforms[BoneIdx] = FTransform3f::Identity;
		}
	}

	bool FCollectionTransformFacade::HasCycle(const TManagedArray<int32>& Parents, int32 Node)
	{
		const int32 NumParents = Parents.Num();
		int32 WalkNode = Node;
		for (int32 Iters = 0; WalkNode != INDEX_NONE && Iters < NumParents; ++Iters)
		{
			WalkNode = Parents[WalkNode];
		}
		return WalkNode != INDEX_NONE;
	}

	bool FCollectionTransformFacade::HasCycle(const TManagedArray<int32>& Parents, const TArray<int32>& SelectedBones)
	{
		for (int32 Bone : SelectedBones)
		{
			if (HasCycle(Parents, Bone))
			{
				return true;
			}
		}
		return false;
	}

	void FCollectionTransformFacade::ParentTransform(const int32 TransformIndex, const int32 ChildIndex)
	{
		check(Collection);

		TArray<int32> SelectedBones;
		SelectedBones.Add(ChildIndex);
		ParentTransforms(TransformIndex, SelectedBones);
	}

	void FCollectionTransformFacade::ParentTransforms(const int32 TransformIndex, const TArray<int32>& SelectedBones)
	{
		check(Collection);
		TManagedArray<FTransform3f>& Transform = TransformAttribute.Modify();
		TManagedArray<int32>& Parents = ParentAttribute.Modify();
		TManagedArray<TSet<int32>>& Children = ChildrenAttribute.Modify();
		
		if (ensure(-1 <= TransformIndex && TransformIndex < Transform.Num()))
		{
			// pre calculate global positions
			TArray<FTransform3f> GlobalTransform;
			GeometryCollectionAlgo::GlobalMatrices(Transform, Parents, GlobalTransform);

			// append children 
			for (int32 Index = 0; Index < SelectedBones.Num(); Index++)
			{
				int32 BoneIndex = SelectedBones[Index];
				if (ensure(0 <= BoneIndex && BoneIndex < Parents.Num()))
				{
					// remove entry in previous parent
					int32 ParentIndex = Parents[BoneIndex];
					if (ParentIndex != INDEX_NONE)
					{
						if (ensure(0 <= ParentIndex && ParentIndex < Parents.Num()))
						{
							Children[ParentIndex].Remove(BoneIndex);
						}
					}

					// set new parent
					Parents[BoneIndex] = TransformIndex;
				}
			}

			FTransform3f ParentInverse = FTransform3f::Identity;
			if (TransformIndex != INDEX_NONE)
			{
				Children[TransformIndex].Append(SelectedBones);
				ParentInverse = GlobalTransform[TransformIndex].Inverse();
			}

			// move the children to the local space of the transform. 
			for (int32 Index = 0; Index < SelectedBones.Num(); Index++)
			{
				int32 BoneIndex = SelectedBones[Index];
				Transform[BoneIndex] = GlobalTransform[BoneIndex] * ParentInverse;
			}

		}

		// error check for circular dependencies
		ensure(!HasCycle(Parents, TransformIndex));
		ensure(!HasCycle(Parents, SelectedBones));
	}

	void FCollectionTransformFacade::UnparentTransform(const int32 ChildIndex)
	{
		check(Collection);

		if (IsValid())
		{
			int32 NumTransforms = Collection->NumElements(FTransformCollection::TransformGroup);

			if (0 < ChildIndex && ChildIndex < NumTransforms)
			{
				TManagedArray<int32>& Parent = ParentAttribute.Modify();
				TManagedArray<TSet<int32>>& Children = ChildrenAttribute.Modify();

				int32 ParentIndex = Parent[ChildIndex];
				if (0 <= ParentIndex && ParentIndex < NumTransforms)
				{
					Children[ParentIndex].Remove(ChildIndex);
					Parent[ChildIndex] = INDEX_NONE;
				}
			}
		}
	}


	CHAOS_API void FCollectionTransformFacade::EnforceSingleRoot(const FString & RootName)
	{
		check(Collection);

		TArray<int32> Roots = GetRootIndices();
		if (Roots.Num() > 1)
		{
			int32 Idx = TransformAttribute.AddElements(1);
			ParentAttribute.Modify()[Idx] = INDEX_NONE;

			if (HasBoneNameAttribute())
			{
				const TManagedArray<FString>& BoneName = BoneNameAttribute.Get();

				auto MakeUnique = [&BoneName](FString NewName)
				{
					int32 CurrentIndex = 1;
					FString TestName = NewName;
					while (BoneName.Contains(TestName))
					{
						TestName = FString::Printf(TEXT("%s%d"), *NewName, CurrentIndex++);
					}
					return TestName;
				};

				BoneNameAttribute.Modify()[Idx]=MakeUnique(RootName);
			}

			ParentTransforms(Idx, Roots);
		}
	}

	void FCollectionTransformFacade::ComputeLevelsBounds(float& OutLevelsMin, float& OutLevelsMax)
	{
		const TManagedArray<int32>& Levels = Collection->GetAttribute<int32>(FTransformCollection::LevelAttribute, FTransformCollection::TransformGroup);

		int32 LevelsMin = INT_MAX, LevelsMax = -INT_MAX;
		for (int32 Idx = 0; Idx < Levels.Num(); ++Idx)
		{
			const int32 Level = Levels[Idx];
			if (Level < LevelsMin)
			{
				LevelsMin = Level;
			}
			else if (Level > LevelsMax)
			{
				LevelsMax = Level;
			}
		}

		OutLevelsMin = (float)LevelsMin;
		OutLevelsMax = (float)LevelsMax;
	}

}