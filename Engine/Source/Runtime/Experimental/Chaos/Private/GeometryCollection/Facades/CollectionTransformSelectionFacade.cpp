// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/Facades/CollectionTransformSelectionFacade.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/Facades/CollectionHierarchyFacade.h"
#include "Algo/RemoveIf.h"
#include "GeometryCollection/Facades/CollectionMeshFacade.h"
#include "GeometryCollection/Facades/CollectionTransformFacade.h"
#include "GeometryCollection/Facades/CollectionBoundsFacade.h"
#include "GeometryCollection/Facades/CollectionCurveFacade.h"

namespace GeometryCollection::Facades
{
	FCollectionTransformSelectionFacade::FCollectionTransformSelectionFacade(const FManagedArrayCollection& InCollection)
		: ConstCollection(InCollection)
		, ParentAttribute(InCollection, "Parent", FTransformCollection::TransformGroup)
		, ChildrenAttribute(InCollection, "Children", FTransformCollection::TransformGroup)
		, LevelAttribute(InCollection, "Level", FTransformCollection::TransformGroup)
		, SimulationTypeAttribute(InCollection, "SimulationType", FTransformCollection::TransformGroup)
		, TransformToGeometryIndexAttribute(InCollection, "TransformToGeometryIndex", FTransformCollection::TransformGroup)
		, TransformIndexAttribute(InCollection, "TransformIndex", FGeometryCollection::GeometryGroup)
		, VertexStartAttribute(InCollection, "VertexStart", FGeometryCollection::GeometryGroup)
		, VertexCountAttribute(InCollection, "VertexCount", FGeometryCollection::GeometryGroup)
		, FaceStartAttribute(InCollection, "FaceStart", FGeometryCollection::GeometryGroup)
		, FaceCountAttribute(InCollection, "FaceCount", FGeometryCollection::GeometryGroup)
		, IndicesAttribute(InCollection, "Indices", FGeometryCollection::FacesGroup)
	{}

	//
	//  Initialization
	//

	// There is no need for schema for this facade
	void FCollectionTransformSelectionFacade::DefineSchema()
	{
		check(!IsConst());
		check(false);
	}

	bool FCollectionTransformSelectionFacade::IsValid() const
	{
		return ParentAttribute.IsValid() &&
			ChildrenAttribute.IsValid() &&
			LevelAttribute.IsValid() &&
			SimulationTypeAttribute.IsValid() &&
			TransformIndexAttribute.IsValid() &&
			TransformToGeometryIndexAttribute.IsValid();
	}

	bool FCollectionTransformSelectionFacade::IsARootBone(const int32 Index) const
	{
		const TManagedArray<int32>& Parents = ParentAttribute.Get();
		return (Parents[Index] == INDEX_NONE);
	}

	bool FCollectionTransformSelectionFacade::HasSelectedAncestor(const TArray<int32>& InSelection, const int32 Index) const
	{
		if (ParentAttribute.IsValid())
		{
			const TManagedArray<int32>& Parents = ParentAttribute.Get();

			if (Index < 0 || Index >= ParentAttribute.Num())
			{
				return false;
			}

			int32 CurrIndex = Index;
			while (Parents[CurrIndex] != INDEX_NONE)
			{
				CurrIndex = Parents[CurrIndex];
				if (InSelection.Contains(CurrIndex))
				{
					return true;
				}
			}
		}

		// We've arrived at the top of the hierarchy with no selected ancestors
		return false;
	}

	void FCollectionTransformSelectionFacade::RemoveRootNodes(TArray<int32>& InOutSelection) const
	{
		if (ParentAttribute.IsValid())
		{
			// Ensure that selected indices are valid
			const int32 NumTransforms = ParentAttribute.Num();

			InOutSelection.RemoveAll([this, NumTransforms](int32 Index) {
				return Index == INDEX_NONE || !ensure(Index < NumTransforms);
				});

			InOutSelection.RemoveAll([this](int32 Index) {
				return IsARootBone(Index);
				});
		}
	}

	// Originally was called GetBonesToLevel() in Geometry library
	TArray<int32> FCollectionTransformSelectionFacade::GetBonesByLevel(const int32 Level, bool bOnlyClusteredOrRigid, bool bSkipFiltered) const
	{
		TArray<int32> OutBones;

		if (LevelAttribute.IsValid() && SimulationTypeAttribute.IsValid() && ChildrenAttribute.IsValid())
		{
			const TManagedArray<int32>& Levels = LevelAttribute.Get();
			const TManagedArray<int32>& SimType = SimulationTypeAttribute.Get();
			const TManagedArray<TSet<int32>>& Children = ChildrenAttribute.Get();

			bool bAllLevels = Level == INDEX_NONE;

			const int32 NumTransforms = ParentAttribute.Num();
			for (int32 BoneIdx = 0; BoneIdx < NumTransforms; BoneIdx++)
			{
				bool bIsRigid = SimType[BoneIdx] == FGeometryCollection::ESimulationTypes::FST_Rigid;
				bool bIsClustered = SimType[BoneIdx] == FGeometryCollection::ESimulationTypes::FST_Clustered;
				if (
					// (if skipping embedded) sim type is clustered or rigid 
					(!bOnlyClusteredOrRigid || bIsClustered || bIsRigid)
					&&
					// (if skipping nodes the outliner has filtered) sim type is clustered or level is an exact match or level has an exact-match child
					(bAllLevels || !bSkipFiltered || bIsClustered || Levels[BoneIdx] == Level || (Children[BoneIdx].Num() > 0 && Levels[BoneIdx] + 1 == Level))
					&&
					// level is at or before the target
					(bAllLevels || Levels[BoneIdx] <= Level)
					)
				{
					OutBones.Add(BoneIdx);
				}
			}
		}

		return OutBones;
	}

	TArray<int32> FCollectionTransformSelectionFacade::GetBonesExactlyAtLevel(const int32 Level, bool bOnlyClusteredOrRigid) const
	{
		TArray<int32> OutBones;

		if (LevelAttribute.IsValid() && SimulationTypeAttribute.IsValid())
		{
			const TManagedArray<int32>& Levels = LevelAttribute.Get();
			bool bHasSimType = SimulationTypeAttribute.IsValid();

			if (!bHasSimType)
			{
				// cannot filter by cluster/rigid if simulation type is not available
				ensure(!bOnlyClusteredOrRigid);
				bOnlyClusteredOrRigid = false;
			}

			bool bAllLevels = Level == INDEX_NONE;

			const int32 NumTransforms = ParentAttribute.Num();
			for (int32 BoneIdx = 0; BoneIdx < NumTransforms; BoneIdx++)
			{
				bool bIsRigid = bHasSimType && SimulationTypeAttribute[BoneIdx] == FGeometryCollection::ESimulationTypes::FST_Rigid;
				bool bIsClustered = bHasSimType && SimulationTypeAttribute[BoneIdx] == FGeometryCollection::ESimulationTypes::FST_Clustered;
				if (
					// (if skipping embedded) sim type is clustered or rigid 
					(!bOnlyClusteredOrRigid || bIsClustered || bIsRigid)
					&&
					// level is at or before the target
					(bAllLevels || Levels[BoneIdx] == Level)
					)
				{
					OutBones.Add(BoneIdx);
				}
			}
		}

		return OutBones;
	}

	void FCollectionTransformSelectionFacade::Sanitize(TArray<int32>& InOutSelection, bool bFavorParents) const
	{
		if (ParentAttribute.IsValid())
		{
			const int32 NumTransforms = ParentAttribute.Num();

			InOutSelection.RemoveAll([this, NumTransforms](int32 Index) {
				return Index == INDEX_NONE || !ensure(Index < NumTransforms);
				});

			// Ensure that children of a selected node are not also selected.
			if (bFavorParents)
			{
				InOutSelection.RemoveAll([this, InOutSelection](int32 Index) {
					return !IsValidBone(Index) || HasSelectedAncestor(InOutSelection, Index);
					});
			}

			InOutSelection.Sort();
		}
	}

	void FCollectionTransformSelectionFacade::ConvertSelectionToRigidNodes(const int32 Index, TArray<int32>& InOutSelection) const
	{
		if (ChildrenAttribute.IsValid() && SimulationTypeAttribute.IsValid())
		{
			const TManagedArray<TSet<int32>>& Children = ChildrenAttribute.Get();
			const TManagedArray<int32>& SimulationType = SimulationTypeAttribute.Get();

			if (SimulationType[Index] == FGeometryCollection::ESimulationTypes::FST_Rigid)
			{
				InOutSelection.Add(Index);
			}
			else
			{
				for (int32 Child : Children[Index])
				{
					ConvertSelectionToRigidNodes(Child, InOutSelection);
				}
			}
		}
	}

	void FCollectionTransformSelectionFacade::ConvertSelectionToRigidNodes(TArray<int32>& InOutSelection) const
	{
		Sanitize(InOutSelection);

		TArray<int32> RigidSelection;
		for (int32 Index : InOutSelection)
		{
			ConvertSelectionToRigidNodes(Index, RigidSelection);
		}

		InOutSelection = RigidSelection;
	}

	void FCollectionTransformSelectionFacade::ConvertEmbeddedSelectionToParents(TArray<int32>& InOutSelection) const
	{
		if (!SimulationTypeAttribute.IsValid()) // if no simulation type, no embedded to convert
		{
			return;
		}

		const TManagedArray<int32>& Parents = ParentAttribute.Get();
		const TManagedArray<int32>& SimulationType = SimulationTypeAttribute.Get();

		Sanitize(InOutSelection);

		for (int32 SelBoneIdx = 0; SelBoneIdx < InOutSelection.Num(); ++SelBoneIdx)
		{
			int32 Bone = InOutSelection[SelBoneIdx];
			if (SimulationType[Bone] == FGeometryCollection::ESimulationTypes::FST_None)
			{
				int32 Parent = Parents[Bone];
				if (Parent != INDEX_NONE)
				{
					InOutSelection[SelBoneIdx] = Parent;
				}
				else // embedded should always have a parent, but if it somehow does not, just remove from selection
				{
					InOutSelection.RemoveAtSwap(SelBoneIdx, EAllowShrinking::No);
					--SelBoneIdx; // reconsider swapped-in-element at this idx next iter
				}
			}
		}
	}

	void FCollectionTransformSelectionFacade::FilterSelectionBySimulationType(TArray<int32>& InOutSelection, FGeometryCollection::ESimulationTypes KeepSimulationType) const
	{
		const TManagedArray<int32>& SimulationType = SimulationTypeAttribute.Get();

		InOutSelection.SetNum(Algo::RemoveIf(InOutSelection, [&](int32 BoneIdx)
			{
				return SimulationType[BoneIdx] != KeepSimulationType;
			}));
	}

	void FCollectionTransformSelectionFacade::ConvertSelectionToClusterNodes(TArray<int32>& InOutSelection, bool bLeaveRigidRoots) const
	{
		const TManagedArray<int32>& Parents = ParentAttribute.Get();
		const TManagedArray<int32>& SimulationType = SimulationTypeAttribute.Get();

		Sanitize(InOutSelection);

		TSet<int32> ClusterNodes;
		for (int32 Index : InOutSelection)
		{
			int32 SimType = SimulationType[Index];
			if (SimType == FGeometryCollection::ESimulationTypes::FST_Clustered)
			{
				ClusterNodes.Add(Index);
			}
			else // if Index is not a cluster, select the cluster containing Index
			{
				int32 Parent = Parents[Index];
				int32 CouldBeRoot = Index;
				if (SimType == FGeometryCollection::ESimulationTypes::FST_None && Parent != -1)
				{
					CouldBeRoot = Parent;
					Parent = Parents[Parent];
				}
				// Special case handling for the rigid root case; only discard from selection if !bLeaveRigidRoots
				if (Parent == -1 && SimulationType[CouldBeRoot] == FGeometryCollection::ESimulationTypes::FST_Rigid)
				{
					if (bLeaveRigidRoots)
					{
						ClusterNodes.Add(Index);
					}
					continue;
				}
				else if (Parent != FGeometryCollection::Invalid && SimulationType[Parent] == FGeometryCollection::ESimulationTypes::FST_Clustered)
				{
					ClusterNodes.Add(Parent);
				}
			}
		}
		InOutSelection = ClusterNodes.Array();
	}

	TArray<int32> FCollectionTransformSelectionFacade::SelectRootBones() const
	{
		TArray<int32> OutSelection;

		if (CanSelectRootBones())
		{
			const TManagedArray<int32>& Parents = ParentAttribute.Get();
			const int32 NumTransforms = ParentAttribute.Num();

			for (int32 Idx = 0; Idx < Parents.Num(); ++Idx)
			{
				if (Parents[Idx] == INDEX_NONE)
				{
					OutSelection.Add(Idx);
				}
			}
		}
		
		return OutSelection;
	}

	TArray<int32> FCollectionTransformSelectionFacade::SelectNone() const
	{
		TArray<int32> OutSelection;
		return OutSelection;

	}

	TArray<int32> FCollectionTransformSelectionFacade::SelectAll() const
	{
		TArray<int32> OutSelection;

		if (CanSelectAll())
		{
			const TManagedArray<int32>& Parents = ParentAttribute.Get();
			const int32 NumTransforms = ParentAttribute.Num();

			for (int32 Idx = 0; Idx < Parents.Num(); ++Idx)
			{
				OutSelection.Add(Idx);
			}
		}
		
		return OutSelection;
	}

	void FCollectionTransformSelectionFacade::SelectInverse(TArray<int32>& InOutSelection) const
	{
		TArray<int32> NewSelection;

		if (CanSelectInverse())
		{
			const TManagedArray<int32>& Parents = ParentAttribute.Get();
			const int32 NumTransforms = ParentAttribute.Num();

			for (int32 Idx = 0; Idx < NumTransforms; ++Idx)
			{
				if (!InOutSelection.Contains(Idx))
				{
					NewSelection.Add(Idx);
				}
			}
		}
		
		InOutSelection = NewSelection;
	}

	TArray<int32> FCollectionTransformSelectionFacade::SelectRandom(bool bDeterministic, int32 RandomSeed, float RandomThresholdVal) const
	{
		TArray<int32> OutSelection;

		if (CanSelectRandom())
		{
			const TManagedArray<int32>& Parents = ParentAttribute.Get();
			const int32 NumTransforms = ParentAttribute.Num();

			FRandomStream Stream(RandomSeed);

			for (int32 Idx = 0; Idx < NumTransforms; ++Idx)
			{
				if (bDeterministic)
				{
					float RandomVal = (float)Stream.FRandRange(0.f, 1.f);

					if (RandomVal < RandomThresholdVal)
					{
						OutSelection.Add(Idx);
					}
				}
				else
				{
					float RandomVal = FMath::FRandRange(0.f, 1.f);

					if (RandomVal < RandomThresholdVal)
					{
						OutSelection.Add(Idx);
					}
				}
			}
		}

		return OutSelection;
	}

	TArray<int32> FCollectionTransformSelectionFacade::SelectRandom(const int32 NumElems, const bool bDeterministic, const int32 RandomSeed, const float RandomThresholdVal) const
	{
		TArray<int32> OutSelection;

		FRandomStream Stream(RandomSeed);

		for (int32 Idx = 0; Idx < NumElems; ++Idx)
		{
			if (bDeterministic)
			{
				float RandomVal = (float)Stream.FRandRange(0.f, 1.f);

				if (RandomVal < RandomThresholdVal)
				{
					OutSelection.Add(Idx);
				}
			}
			else
			{
				float RandomVal = FMath::FRandRange(0.f, 1.f);

				if (RandomVal < RandomThresholdVal)
				{
					OutSelection.Add(Idx);
				}
			}
		}

		return OutSelection;
	}

	TArray<int32> FCollectionTransformSelectionFacade::SelectLeaf() const
	{
		TArray<int32> OutSelection;

		if (CanSelectLeaf())
		{
			const TManagedArray<int32>& SimType = SimulationTypeAttribute.Get();

			const int32 NumTransforms = SimType.Num();
			for (int32 BoneIdx = 0; BoneIdx < NumTransforms; BoneIdx++)
			{
				bool bIsRigid = SimType[BoneIdx] == FGeometryCollection::ESimulationTypes::FST_Rigid;
				if (bIsRigid)
				{
					OutSelection.Add(BoneIdx);
				}
			}
		}
		
		return OutSelection;
	}

	TArray<int32> FCollectionTransformSelectionFacade::SelectCluster() const
	{
		TArray<int32> OutSelection;

		if (CanSelectCluster())
		{
			const TManagedArray<int32>& SimType = SimulationTypeAttribute.Get();

			const int32 NumTransforms = SimType.Num();
			for (int32 BoneIdx = 0; BoneIdx < NumTransforms; BoneIdx++)
			{
				bool bIsCluster = SimType[BoneIdx] == FGeometryCollection::ESimulationTypes::FST_Clustered;
				if (bIsCluster)
				{
					OutSelection.Add(BoneIdx);
				}
			}
		}
		
		return OutSelection;
	}

	void FCollectionTransformSelectionFacade::SelectContact(TArray<int32>& InOutSelection, bool bIncludeNeighborsInParentLevels) const
	{
		TSet<int32> NewSelection;

		if (CanSelectContact())
		{
			if (TUniquePtr<FGeometryCollection> TempGeomCollection = TUniquePtr<FGeometryCollection>(ConstCollection.NewCopy<FGeometryCollection>()))
			{
				FGeometryCollectionProximityUtility ProximityUtility(TempGeomCollection.Get());
				ProximityUtility.RequireProximity();

				const TManagedArray<int32>& TransformIndex = TempGeomCollection->TransformIndex;
				const TManagedArray<int32>& TransformToGeometryIndex = TempGeomCollection->TransformToGeometryIndex;
				const TManagedArray<TSet<int32>>& Proximity = TempGeomCollection->GetAttribute<TSet<int32>>("Proximity", FGeometryCollection::GeometryGroup);
				Chaos::Facades::FCollectionHierarchyFacade HierarchyFacade(*TempGeomCollection);

				for (int32 Bone : InOutSelection)
				{
					NewSelection.Add(Bone);
					ProximityUtility.EnumerateNeighbors(HierarchyFacade, Bone, [&NewSelection](int32 NeighborTransformIdx)
						{
							NewSelection.Add(NeighborTransformIdx);
						}, bIncludeNeighborsInParentLevels, false /*bFilterDuplicates, not needed since we add to a set*/);
				}
			}
		}

		InOutSelection = NewSelection.Array();
	}

	void FCollectionTransformSelectionFacade::SelectParent(TArray<int32>& InOutSelection) const
	{
		TSet<int32> NewSelection;

		if (CanSelectParent())
		{
			const TManagedArray<int32>& Parents = ParentAttribute.Get();

			for (int32 Bone : InOutSelection)
			{
				int32 ParentBone = Parents[Bone];
				if (ParentBone != FGeometryCollection::Invalid)
				{
					NewSelection.Add(ParentBone);
				}
			}
		}
		
		InOutSelection = NewSelection.Array();
	}

	void FCollectionTransformSelectionFacade::SelectChildren(TArray<int32>& InOutSelection) const
	{
		TSet<int32> NewSelection;

		if (CanSelectChildren())
		{
			const TManagedArray<TSet<int32>>& Children = ChildrenAttribute.Get();

			for (int32 Bone : InOutSelection)
			{
				if (Children[Bone].IsEmpty())
				{
					NewSelection.Add(Bone);
					continue;
				}
				for (int32 Child : Children[Bone])
				{
					NewSelection.Add(Child);
				}
			}
		}

		InOutSelection = NewSelection.Array();
	}

	void FCollectionTransformSelectionFacade::SelectSiblings(TArray<int32>& InOutSelection) const
	{
		TSet<int32> NewSelection;

		if (CanSelectSiblings())
		{
			const TManagedArray<int32>& Parents = ParentAttribute.Get();
			const TManagedArray<TSet<int32>>& Children = ChildrenAttribute.Get();

			for (int32 Bone : InOutSelection)
			{
				int32 ParentBone = Parents[Bone];
				if (ParentBone != INDEX_NONE)
				{
					for (int32 Child : Children[ParentBone])
					{
						NewSelection.Add(Child);
					}
				}
			}
		}
		
		InOutSelection = NewSelection.Array();
	}

	void FCollectionTransformSelectionFacade::SelectLevel(TArray<int32>& InOutSelection) const
	{
		TSet<int32> NewSelection;

		if (CanSelectLevel())
		{
			const TManagedArray<int32>& Levels = LevelAttribute.Get();
			const int32 NumTransforms = ParentAttribute.Num();

			for (int32 Bone : InOutSelection)
			{
				int32 Level = Levels[Bone];
				for (int32 TransformIdx = 0; TransformIdx < NumTransforms; ++TransformIdx)
				{
					if (Levels[TransformIdx] == Level)
					{
						NewSelection.Add(TransformIdx);
					}
				}
			}
		}
		
		InOutSelection = NewSelection.Array();
	}

	static void RandomShuffleArray(TArray<int32>& InArray, bool Deterministic, int32 RandomSeed)
	{
		FRandomStream Stream(RandomSeed);

		int32 LastIndex = InArray.Num() - 1;

		static const int32 NumIterationsMin = 10;
		static const int32 NumIterationsMax = 50;

		int32 NumIterations = Deterministic ? Stream.RandRange(NumIterationsMin, NumIterationsMax) : FMath::RandRange(NumIterationsMin, NumIterationsMax);

		for (int32 IterIdx = 0; IterIdx < NumIterations; ++IterIdx)
		{
			for (int32 Idx = 0; Idx <= LastIndex; ++Idx)
			{
				int32 Index;
				if (Deterministic)
				{
					Index = Stream.RandRange(Idx, LastIndex);
				}
				else
				{
					Index = FMath::RandRange(Idx, LastIndex);
				}

				if (Idx != Index)
				{
					InArray.Swap(Idx, Index);
				}
			}
		}
	}

	void FCollectionTransformSelectionFacade::SelectByPercentage(TArray<int32>& InOutSelection, int32 InPercentage, bool Deterministic, int32 RandomSeed)
	{
		RandomShuffleArray(InOutSelection, Deterministic, RandomSeed);

		if (!FMath::IsWithinInclusive<int32>(InPercentage, 0, 100))
		{
			InPercentage = 100;
		}

		float Percentage = (float)InPercentage * 0.01f;
		int32 NewNumElements = FMath::RoundToInt32((float)InOutSelection.Num() * Percentage);
		
		InOutSelection.SetNum(NewNumElements);
	}

	TArray<int32> FCollectionTransformSelectionFacade::SelectBySize(float SizeMin, float SizeMax, bool bInclusive, bool bInsideRange, bool bUseRelativeSize) const
	{
		// if not using relative size, convert sizes to volumes and select by volume
		if (!bUseRelativeSize)
		{
			float VolumeMin = SizeMin * SizeMin * SizeMin;
			float VolumeMax = SizeMax * SizeMax * SizeMax;
			return SelectByVolume(VolumeMin, VolumeMax, bInclusive, bInsideRange);
		}

		TArray<int32> OutSelection;

		// TODO: (See also SelectByVolume) We should add a method to get the volumes without modifying the collection, and then remove this full geometrycollection copy
		if (TUniquePtr<FGeometryCollection> TempGeomCollection = TUniquePtr<FGeometryCollection>(ConstCollection.NewCopy<FGeometryCollection>()))
		{
			FGeometryCollectionConvexUtility::SetVolumeAttributes(TempGeomCollection.Get());

			const TManagedArray<float>* SizesPtr = TempGeomCollection->FindAttribute<float>("Size", FTransformCollection::TransformGroup);
			if (!ensure(SizesPtr))
			{
				return OutSelection;
			}
			const TManagedArray<float>& Sizes = *SizesPtr;

			for (int32 BoneIdx = 0; BoneIdx < Sizes.Num(); ++BoneIdx)
			{
				const float Size = Sizes[BoneIdx];

				if (bInsideRange)
				{
					if (bInclusive)
					{
						if (Size >= SizeMin && Size <= SizeMax)
						{
							OutSelection.Add(BoneIdx);
						}
					}
					else
					{
						if (Size > SizeMin && Size < SizeMax)
						{
							OutSelection.Add(BoneIdx);
						}
					}
				}
				else
				{
					if (bInclusive)
					{
						if (Size <= SizeMin || Size >= SizeMax)
						{
							OutSelection.Add(BoneIdx);
						}
					}
					else
					{
						if (Size < SizeMin || Size > SizeMax)
						{
							OutSelection.Add(BoneIdx);
						}
					}
				}
			}
		}

		return OutSelection;
	}

	TArray<int32> FCollectionTransformSelectionFacade::SelectByVolume(float VolumeMin, float VolumeMax, bool bInclusive, bool bInsideRange) const
	{
		TArray<int32> OutSelection;

		// TODO: (See also SelectBySize) We should add a method to get the volumes without modifying the collection, and then remove this full geometrycollection copy
		if (TUniquePtr<FGeometryCollection> TempGeomCollection = TUniquePtr<FGeometryCollection>(ConstCollection.NewCopy<FGeometryCollection>()))
		{
			FGeometryCollectionConvexUtility::SetVolumeAttributes(TempGeomCollection.Get());

			const TManagedArray<float>* VolumesPtr = TempGeomCollection->FindAttribute<float>("Volume", FTransformCollection::TransformGroup);
			if (!ensure(VolumesPtr))
			{
				return OutSelection;
			}
			const TManagedArray<float>& Volumes = *VolumesPtr;

			for (int32 BoneIdx = 0; BoneIdx < Volumes.Num(); ++BoneIdx)
			{
				const float Volume = Volumes[BoneIdx];

				if (bInsideRange)
				{
					if (bInclusive)
					{
						if (Volume >= VolumeMin && Volume <= VolumeMax)
						{
							OutSelection.Add(BoneIdx);
						}
					}
					else
					{
						if (Volume > VolumeMin && Volume < VolumeMax)
						{
							OutSelection.Add(BoneIdx);
						}
					}
				}
				else
				{
					if (bInclusive)
					{
						if (Volume <= VolumeMin || Volume >= VolumeMax)
						{
							OutSelection.Add(BoneIdx);
						}
					}
					else
					{
						if (!(Volume < VolumeMin || Volume > VolumeMax))
						{
							OutSelection.Add(BoneIdx);
						}
					}
				}
			}
		}

		return OutSelection;
	}

	TMap<int32, TArray<int32>> FCollectionTransformSelectionFacade::GetClusteredSelections(const TArray<int32>& InSelection) const
	{
		TMap<int32, TArray<int32>> SiblingGroups;

		// Bin the selection indices by parent index
		const TManagedArray<int32>& Parents = ParentAttribute.Get();
		for (int32 Index : InSelection)
		{
			TArray<int32>& SiblingIndices = SiblingGroups.FindOrAdd(Parents[Index]);
			SiblingIndices.Add(Index);
		}

		return SiblingGroups;
	}

	/* --------------------------------------------------------------------------------------------------------------------------- */

	void FCollectionTransformSelectionFacade::SelectVerticesInTransformedBox(const FBox& InBox, const FTransform& InBoxTransform, TArray<bool>& OutVertexSelection) const
	{
		const FCollectionMeshFacade MeshFacade(ConstCollection);
		const FCollectionTransformFacade TransformFacade(ConstCollection);

		OutVertexSelection.Reset();
		if (!MeshFacade.IsValid() || !TransformFacade.IsValid())
		{
			return;
		}
		const int32 NumVertices = MeshFacade.VertexAttribute.Num();
		OutVertexSelection.Reserve(NumVertices);

		const TArray<FTransform> CollectionSpaceTransforms = TransformFacade.ComputeCollectionSpaceTransforms();

		const FVector BoxCenter = InBox.GetCenter();
		const FVector BoxExtentsScaled = InBox.GetExtent() * InBoxTransform.GetScale3D();
		const FBox ScaledBoxCenterAtZero(-BoxExtentsScaled, BoxExtentsScaled);

		for (int32 VertexIdx = 0; VertexIdx < NumVertices; ++VertexIdx)
		{
			const int32 TransformIdx = MeshFacade.BoneMapAttribute.IsValidIndex(VertexIdx) ? MeshFacade.BoneMapAttribute[VertexIdx] : INDEX_NONE;

			bool bIsSelected = false;
			if (CollectionSpaceTransforms.IsValidIndex(TransformIdx))
			{
				const FVector VertexInBoneSpace = FVector(MeshFacade.VertexAttribute[VertexIdx]);
				const FVector VertexInCollectionSpace = CollectionSpaceTransforms[TransformIdx].TransformPosition(VertexInBoneSpace);
				const FVector VertexInBoxSpace = InBoxTransform.InverseTransformPositionNoScale(VertexInCollectionSpace - BoxCenter);

				bIsSelected = ScaledBoxCenterAtZero.IsInside(VertexInBoxSpace);
			}
			OutVertexSelection.Add(bIsSelected);
		}
	}
	
	/* --------------------------------------------------------------------------------------------------------------------------- */

	void FCollectionTransformSelectionFacade::SelectVerticesInTransformedBox(const FBox& InBox, const FTransform& InBoxTransform, TBitArray<>& OutVertexSelection) const
	{
		const FCollectionMeshFacade MeshFacade(ConstCollection);
		const FCollectionTransformFacade TransformFacade(ConstCollection);

		OutVertexSelection.Reset();
		if (!MeshFacade.IsValid() || !TransformFacade.IsValid())
		{
			return;
		}
		const int32 NumVertices = MeshFacade.VertexAttribute.Num();
		OutVertexSelection.Reserve(NumVertices);
		OutVertexSelection.SetNum(NumVertices, false);

		const TArray<FTransform> CollectionSpaceTransforms = TransformFacade.ComputeCollectionSpaceTransforms();

		const FVector BoxCenter = InBox.GetCenter();
		const FVector BoxExtentsScaled = InBox.GetExtent() * InBoxTransform.GetScale3D();
		const FBox ScaledBoxCenterAtZero(-BoxExtentsScaled, BoxExtentsScaled);

		for (int32 VertexIdx = 0; VertexIdx < NumVertices; ++VertexIdx)
		{
			const int32 TransformIdx = MeshFacade.BoneMapAttribute.IsValidIndex(VertexIdx) ? MeshFacade.BoneMapAttribute[VertexIdx] : INDEX_NONE;

			bool bIsSelected = false;
			if (CollectionSpaceTransforms.IsValidIndex(TransformIdx))
			{
				const FVector VertexInBoneSpace = FVector(MeshFacade.VertexAttribute[VertexIdx]);
				const FVector VertexInCollectionSpace = CollectionSpaceTransforms[TransformIdx].TransformPosition(VertexInBoneSpace);
				const FVector VertexInBoxSpace = InBoxTransform.InverseTransformPositionNoScale(VertexInCollectionSpace - BoxCenter);

				bIsSelected = ScaledBoxCenterAtZero.IsInside(VertexInBoxSpace);
			}
			OutVertexSelection[VertexIdx] = bIsSelected;
		}
	}

	/* --------------------------------------------------------------------------------------------------------------------------- */

	void FCollectionTransformSelectionFacade::SelectVerticesInSphere(const FSphere& InSphere, TArray<bool>& OutVertexSelection) const
	{
		const FCollectionMeshFacade MeshFacade(ConstCollection);
		const FCollectionTransformFacade TransformFacade(ConstCollection);

		OutVertexSelection.Reset();
		if (!MeshFacade.IsValid() || !TransformFacade.IsValid())
		{
			return;
		}
		const int32 NumVertices = MeshFacade.VertexAttribute.Num();
		OutVertexSelection.Reserve(NumVertices);

		const TArray<FTransform> CollectionSpaceTransforms = TransformFacade.ComputeCollectionSpaceTransforms();

		for (int32 VertexIdx = 0; VertexIdx < NumVertices; ++VertexIdx)
		{
			const int32 TransformIdx = MeshFacade.BoneMapAttribute.IsValidIndex(VertexIdx) ? MeshFacade.BoneMapAttribute[VertexIdx] : INDEX_NONE;

			bool bIsSelected = false;
			if (CollectionSpaceTransforms.IsValidIndex(TransformIdx))
			{
				const FVector VertexInBoneSpace = FVector(MeshFacade.VertexAttribute[VertexIdx]);
				const FVector VertexInCollectionSpace = CollectionSpaceTransforms[TransformIdx].TransformPosition(VertexInBoneSpace);

				bIsSelected = InSphere.IsInside(VertexInCollectionSpace);
			}
			OutVertexSelection.Add(bIsSelected);
		}
	}

	/* --------------------------------------------------------------------------------------------------------------------------- */

	void FCollectionTransformSelectionFacade::SelectVerticesInSphere(const FSphere& InSphere, TBitArray<>& OutVertexSelection) const
	{
		const FCollectionMeshFacade MeshFacade(ConstCollection);
		const FCollectionTransformFacade TransformFacade(ConstCollection);

		OutVertexSelection.Reset();
		if (!MeshFacade.IsValid() || !TransformFacade.IsValid())
		{
			return;
		}
		const int32 NumVertices = MeshFacade.VertexAttribute.Num();
		OutVertexSelection.Reserve(NumVertices);
		OutVertexSelection.SetNum(NumVertices, false);

		const TArray<FTransform> CollectionSpaceTransforms = TransformFacade.ComputeCollectionSpaceTransforms();

		for (int32 VertexIdx = 0; VertexIdx < NumVertices; ++VertexIdx)
		{
			const int32 TransformIdx = MeshFacade.BoneMapAttribute.IsValidIndex(VertexIdx) ? MeshFacade.BoneMapAttribute[VertexIdx] : INDEX_NONE;

			bool bIsSelected = false;
			if (CollectionSpaceTransforms.IsValidIndex(TransformIdx))
			{
				const FVector VertexInBoneSpace = FVector(MeshFacade.VertexAttribute[VertexIdx]);
				const FVector VertexInCollectionSpace = CollectionSpaceTransforms[TransformIdx].TransformPosition(VertexInBoneSpace);

				bIsSelected = InSphere.IsInside(VertexInCollectionSpace);
			}
			OutVertexSelection[VertexIdx] = bIsSelected;
		}
	}

	/* --------------------------------------------------------------------------------------------------------------------------- */

	void FCollectionTransformSelectionFacade::SelectVerticesOnPlaneSide(const FPlane& InPlane, bool bUsePositiveSide, TArray<bool>& OutVertexSelection) const
	{
		const FCollectionMeshFacade MeshFacade(ConstCollection);
		const FCollectionTransformFacade TransformFacade(ConstCollection);

		OutVertexSelection.Reset();
		if (!MeshFacade.IsValid() || !TransformFacade.IsValid())
		{
			return;
		}
		const int32 NumVertices = MeshFacade.VertexAttribute.Num();
		OutVertexSelection.Reserve(NumVertices);

		const TArray<FTransform> CollectionSpaceTransforms = TransformFacade.ComputeCollectionSpaceTransforms();

		for (int32 VertexIdx = 0; VertexIdx < NumVertices; ++VertexIdx)
		{
			const int32 TransformIdx = MeshFacade.BoneMapAttribute.IsValidIndex(VertexIdx)? MeshFacade.BoneMapAttribute[VertexIdx]: INDEX_NONE;

			bool bIsSelected = false;
			if (CollectionSpaceTransforms.IsValidIndex(TransformIdx))
			{
				const FVector VertexInBoneSpace = FVector(MeshFacade.VertexAttribute[VertexIdx]);
				const FVector VertexInCollectionSpace = CollectionSpaceTransforms[TransformIdx].TransformPosition(VertexInBoneSpace);

				const double DistanceFromPlane = InPlane.PlaneDot(VertexInCollectionSpace);
				bIsSelected = bUsePositiveSide
					? (DistanceFromPlane >= 0)
					: (DistanceFromPlane <= 0);
			}
			OutVertexSelection.Add(bIsSelected);
		}
	}

	/* --------------------------------------------------------------------------------------------------------------------------- */

	void FCollectionTransformSelectionFacade::SelectVerticesOnPlaneSide(const FPlane& InPlane, bool bUsePositiveSide, TBitArray<>& OutVertexSelection) const
	{
		const FCollectionMeshFacade MeshFacade(ConstCollection);
		const FCollectionTransformFacade TransformFacade(ConstCollection);

		OutVertexSelection.Reset();
		if (!MeshFacade.IsValid() || !TransformFacade.IsValid())
		{
			return;
		}
		const int32 NumVertices = MeshFacade.VertexAttribute.Num();
		OutVertexSelection.Reserve(NumVertices);
		OutVertexSelection.SetNum(NumVertices, false);

		const TArray<FTransform> CollectionSpaceTransforms = TransformFacade.ComputeCollectionSpaceTransforms();

		for (int32 VertexIdx = 0; VertexIdx < NumVertices; ++VertexIdx)
		{
			const int32 TransformIdx = MeshFacade.BoneMapAttribute.IsValidIndex(VertexIdx) ? MeshFacade.BoneMapAttribute[VertexIdx] : INDEX_NONE;

			bool bIsSelected = false;
			if (CollectionSpaceTransforms.IsValidIndex(TransformIdx))
			{
				const FVector VertexInBoneSpace = FVector(MeshFacade.VertexAttribute[VertexIdx]);
				const FVector VertexInCollectionSpace = CollectionSpaceTransforms[TransformIdx].TransformPosition(VertexInBoneSpace);

				const double DistanceFromPlane = InPlane.PlaneDot(VertexInCollectionSpace);
				bIsSelected = bUsePositiveSide
					? (DistanceFromPlane >= 0)
					: (DistanceFromPlane <= 0);
			}
			OutVertexSelection[VertexIdx] = bIsSelected;
		}
	}

	/* --------------------------------------------------------------------------------------------------------------------------- */

	TArray<int32> FCollectionTransformSelectionFacade::SelectVerticesInBox(const FBox& InBox, const FTransform& InBoxTransform, bool bAllVerticesInBox) const
	{
		TArray<int32> OutSelection;

		if (CanSelectVerticesInBox())
		{
			const int32 NumTransforms = ParentAttribute.Num();

			FCollectionMeshFacade MeshFacade(ConstCollection);
			FCollectionTransformFacade TransformFacade(ConstCollection);

			for (int32 TransformIdx = 0; TransformIdx < NumTransforms; ++TransformIdx)
			{
				const FTransform CollectionSpaceTransform = TransformFacade.ComputeCollectionSpaceTransform(TransformIdx);

				int32 NumVerticesInsideBox = 0;

				const TArrayView<const FVector3f> VertexPositions = MeshFacade.GetVertexPositions(TransformIdx);
				for (const FVector3f& Vertex : VertexPositions)
				{
					const FVector VertexInBoneSpace(Vertex);

					// Transform from BoneSpace to CollectionSpace
					const FVector VertexInCollectionSpace = CollectionSpaceTransform.TransformPosition(VertexInBoneSpace);

					// Transform in Box space 
					const FVector VertexInBoxSpace = InBoxTransform.InverseTransformPosition(VertexInCollectionSpace);

					if (!InBox.IsInside(VertexInBoxSpace))
					{
						break;
					}

					NumVerticesInsideBox++;
					if (!bAllVerticesInBox)
					{
						break;
					}
				}

				if ((bAllVerticesInBox && NumVerticesInsideBox == VertexPositions.Num()) ||
					(!bAllVerticesInBox && NumVerticesInsideBox > 0))
				{
					OutSelection.Add(TransformIdx);
				}
			}
		}
		
		return OutSelection;
	}

	TArray<int32> FCollectionTransformSelectionFacade::SelectCentroidInBox(const FBox& InBox, const FTransform& InBoxTransform) const
	{
		TArray<int32> OutSelection;

		if (CanSelectCentroidInBox())
		{
			const int32 NumTransforms = ParentAttribute.Num();

			FBoundsFacade BoundsFacade(ConstCollection);
			FCollectionTransformFacade TransformFacade(ConstCollection);

			const TArray<FVector> Centroids = BoundsFacade.GetCentroids();

			for (int32 TransformIdx = 0; TransformIdx < NumTransforms; ++TransformIdx)
			{
				const TManagedArray<int32>& TransformToGeometryIndices = TransformToGeometryIndexAttribute.Get();
				const int32 GeometryIndex = TransformToGeometryIndices[TransformIdx];
				if (Centroids.IsValidIndex(GeometryIndex))
				{
					const FVector CentroidInBoneSpace(Centroids[GeometryIndex]);

					// Transform from BoneSpace to CollectionSpace
					const FTransform CollectionSpaceTransform = TransformFacade.ComputeCollectionSpaceTransform(TransformIdx);
					const FVector CentroidInCollectionSpace = CollectionSpaceTransform.TransformPosition(CentroidInBoneSpace);

					// Transform with specified transform
					const FVector CentroidInBoxSpace = InBoxTransform.InverseTransformPosition(CentroidInCollectionSpace);

					if (InBox.IsInside(CentroidInBoxSpace))
					{
						OutSelection.Add(TransformIdx);
					}
				}
			}
		}
		
		return OutSelection;
	}

	TArray<int32> FCollectionTransformSelectionFacade::SelectBoundingBoxInBox(const FBox& InBox, const FTransform& InBoxTransform) const
	{
		TArray<int32> OutSelection;

		if (CanSelectBoundingBoxInBox())
		{
			const int32 NumTransforms = ParentAttribute.Num();

			FBoundsFacade BoundsFacade(ConstCollection);
			FCollectionTransformFacade TransformFacade(ConstCollection);

			const TManagedArray<FBox>& BoundingBoxes = BoundsFacade.GetBoundingBoxes();

			for (int32 TransformIdx = 0; TransformIdx < NumTransforms; ++TransformIdx)
			{
				const TManagedArray<int32>& TransformToGeometryIndices = TransformToGeometryIndexAttribute.Get();
				const int32 GeometryIndex = TransformToGeometryIndices[TransformIdx];
				if (BoundingBoxes.IsValidIndex(GeometryIndex))
				{
					const FBox& BoundingBoxInBoneSpace = BoundingBoxes[GeometryIndex];

					// Transform from BoneSpace to CollectionSpace
					const FTransform CollectionSpaceTransform = TransformFacade.ComputeCollectionSpaceTransform(TransformIdx);
					const FBox BoundingBoxInCollectionSpace = BoundingBoxInBoneSpace.TransformBy(CollectionSpaceTransform);

					// Transform with specified transform
					const FBox BoundingBoxInBoxSpace = BoundingBoxInCollectionSpace.InverseTransformBy(InBoxTransform);

					if (InBox.IsInside(BoundingBoxInBoxSpace))
					{
						OutSelection.Add(TransformIdx);
					}
				}
			}
		}
		
		return OutSelection;
	}

	TArray<int32> FCollectionTransformSelectionFacade::SelectVerticesInSphere(const FSphere& InSphere, const FTransform& InSphereTransform, bool bAllVerticesInSphere) const
	{
		TArray<int32> OutSelection;

		if (CanSelectVerticesInSphere())
		{
			const int32 NumTransforms = ParentAttribute.Num();

			FCollectionMeshFacade MeshFacade(ConstCollection);
			FCollectionTransformFacade TransformFacade(ConstCollection);

			for (int32 TransformIdx = 0; TransformIdx < NumTransforms; ++TransformIdx)
			{
				int32 NumVerticesInside = 0;

				const TArrayView<const FVector3f> VertexPositions = MeshFacade.GetVertexPositions(TransformIdx);
				for (const FVector3f& Vertex : VertexPositions)
				{
					const FVector VertexInBoneSpace(Vertex);

					// Transform from BoneSpace to CollectionSpace
					const FTransform CollectionSpaceTransform = TransformFacade.ComputeCollectionSpaceTransform(TransformIdx);
					const FVector VertexInCollectionSpace = CollectionSpaceTransform.TransformPosition(VertexInBoneSpace);

					// Transform with specified transform
					const FVector VertexInSphereSpace = InSphereTransform.InverseTransformPosition(VertexInCollectionSpace);

					if (!InSphere.IsInside(VertexInSphereSpace))
					{
						break;
					}

					NumVerticesInside++;
					if (!bAllVerticesInSphere)
					{
						break;
					}
				}

				if ((bAllVerticesInSphere && NumVerticesInside == VertexPositions.Num()) ||
					(!bAllVerticesInSphere && NumVerticesInside > 0))
				{
					OutSelection.Add(TransformIdx);
				}
			}
		}

		return OutSelection;
	}

	TArray<int32> FCollectionTransformSelectionFacade::SelectCentroidInSphere(const FSphere& InSphere, const FTransform& InSphereTransform) const
	{
		TArray<int32> OutSelection;

		if (CanSelectCentroidInSphere())
		{
			const int32 NumTransforms = ParentAttribute.Num();

			FBoundsFacade BoundsFacade(ConstCollection);
			FCollectionTransformFacade TransformFacade(ConstCollection);

			const TArray<FVector> Centroids = BoundsFacade.GetCentroids();

			for (int32 TransformIdx = 0; TransformIdx < NumTransforms; ++TransformIdx)
			{
				const TManagedArray<int32>& TransformToGeometryIndices = TransformToGeometryIndexAttribute.Get();
				const int32 GeometryIndex = TransformToGeometryIndices[TransformIdx];
				if (Centroids.IsValidIndex(GeometryIndex))
				{
					const FVector CentroidInBoneSpace(Centroids[GeometryIndex]);

					// Transform from BoneSpace to CollectionSpace
					const FTransform CollectionSpaceTransform = TransformFacade.ComputeCollectionSpaceTransform(TransformIdx);
					const FVector CentroidInCollectionSpace = CollectionSpaceTransform.TransformPosition(CentroidInBoneSpace);

					// Transform with specified transform
					const FVector CentroidInSphereSpace = InSphereTransform.InverseTransformPosition(CentroidInCollectionSpace);

					if (InSphere.IsInside(CentroidInSphereSpace))
					{
						OutSelection.Add(TransformIdx);
					}
				}
			}
		}
		
		return OutSelection;
	}

	TArray<int32> FCollectionTransformSelectionFacade::SelectBoundingBoxInSphere(const FSphere& InSphere, const FTransform& InSphereTransform) const
	{
		TArray<int32> OutSelection;

		if (CanSelectBoundingBoxInSphere())
		{
			const int32 NumTransforms = ParentAttribute.Num();

			FBoundsFacade BoundsFacade(ConstCollection);
			FCollectionTransformFacade TransformFacade(ConstCollection);

			const TManagedArray<FBox>& BoundingBoxes = BoundsFacade.GetBoundingBoxes();

			for (int32 TransformIdx = 0; TransformIdx < NumTransforms; ++TransformIdx)
			{
				const TManagedArray<int32>& TransformToGeometryIndices = TransformToGeometryIndexAttribute.Get();
				const int32 GeometryIndex = TransformToGeometryIndices[TransformIdx];
				if (BoundingBoxes.IsValidIndex(GeometryIndex))
				{
					const FBox& BoundingBoxInBoneSpace = BoundingBoxes[GeometryIndex];

					int32 NumVerticesInside = 0;

					const TArray<FVector>& VertexInBoneSpaceArr = FBoundsFacade::GetBoundingBoxVertexPositions(BoundingBoxInBoneSpace);
					for (int32 VertexIdx = 0; VertexIdx < VertexInBoneSpaceArr.Num(); ++VertexIdx)
					{
						// Transform from BoneSpace to CollectionSpace
						const FTransform CollectionSpaceTransform = TransformFacade.ComputeCollectionSpaceTransform(TransformIdx);
						const FVector VertexInCollectionSpace = CollectionSpaceTransform.TransformPosition(VertexInBoneSpaceArr[VertexIdx]);

						// Transform with specified transform
						const FVector VertexInSphereSpace = InSphereTransform.InverseTransformPosition(VertexInCollectionSpace);

						if (!InSphere.IsInside(VertexInSphereSpace))
						{
							break;
						}

						NumVerticesInside++;
					}

					if (NumVerticesInside == VertexInBoneSpaceArr.Num())
					{
						OutSelection.Add(TransformIdx);
					}
				}
			}
		}
		
		return OutSelection;
	}

	TArray<int32> FCollectionTransformSelectionFacade::SelectByFloatAttribute(FString GroupName, FString AttrName, float Min, float Max, bool bInclusive, bool bInsideRange) const
	{
		TArray<int32> OutSelection;

		if (const TManagedArray<float>* FloatArray = ConstCollection.FindAttributeTyped<float>(FName(*AttrName), FName(*GroupName)))
		{
			for (int32 Idx = 0; Idx < FloatArray->Num(); ++Idx)
			{
				const float FloatValue = (*FloatArray)[Idx];

				if (bInsideRange)
				{
					if (bInclusive)
					{
						if (FloatValue >= Min && FloatValue <= Max)
						{
							OutSelection.Add(Idx);
						}
					}
					else
					{
						if (FloatValue > Min && FloatValue < Max)
						{
							OutSelection.Add(Idx);
						}
					}
				}
				else
				{
					if (bInclusive)
					{
						if (FloatValue <= Min || FloatValue >= Max)
						{
							OutSelection.Add(Idx);
						}
					}
					else
					{
						if (FloatValue < Min || FloatValue > Max)
						{
							OutSelection.Add(Idx);
						}
					}
				}
			}
		}

		return OutSelection;
	}

	TArray<int32> FCollectionTransformSelectionFacade::SelectByIntAttribute(FString GroupName, FString AttrName, int32 Min, int32 Max, bool bInclusive, bool bInsideRange) const
	{
		TArray<int32> OutSelection;

		if (const TManagedArray<int32>* IntArray = ConstCollection.FindAttributeTyped<int32>(FName(*AttrName), FName(*GroupName)))
		{
			for (int32 Idx = 0; Idx < IntArray->Num(); ++Idx)
			{
				const int32 IntValue = (*IntArray)[Idx];

				if (bInsideRange)
				{
					if (bInclusive)
					{
						if (IntValue >= Min && IntValue <= Max)
						{
							OutSelection.Add(Idx);
						}
					}
					else
					{
						if (IntValue > Min && IntValue < Max)
						{
							OutSelection.Add(Idx);
						}
					}
				}
				else
				{
					if (bInclusive)
					{
						if (IntValue <= Min || IntValue >= Max)
						{
							OutSelection.Add(Idx);
						}
					}
					else
					{
						if (IntValue < Min || IntValue > Max)
						{
							OutSelection.Add(Idx);
						}
					}
				}
			}
		}

		return OutSelection;
	}

	TArray<int32> FCollectionTransformSelectionFacade::ConvertVertexSelectionToTransformSelection(const TArray<int32>& InVertexSelection, bool bAllElementsMustBeSelected) const
	{
		TArray<int32> OutSelection;

		if (TransformIndexAttribute.IsValid() && VertexStartAttribute.IsValid() && VertexCountAttribute.IsValid())
		{
			const TManagedArray<int32>& TransformIndices = TransformIndexAttribute.Get();
			const TManagedArray<int32>& VertexStarts = VertexStartAttribute.Get();
			const TManagedArray<int32>& VertexCounts = VertexCountAttribute.Get();

			const int32 NumGeos = ConstCollection.NumElements(FGeometryCollection::GeometryGroup);

			if (!bAllElementsMustBeSelected)
			{
				for (int32 VertexIdx : InVertexSelection)
				{
					for (int32 GeometryIdx = 0; GeometryIdx < NumGeos; ++GeometryIdx)
					{
						const int32 VertexStart = VertexStarts[GeometryIdx];
						const int32 VertexEnd = VertexStart + VertexCounts[GeometryIdx];

						if (VertexIdx >= VertexStart && VertexIdx < VertexEnd)
						{
							OutSelection.AddUnique(TransformIndices[GeometryIdx]);
							break;
						}
					}
				}
			}
			else
			{
				for (int32 GeometryIdx = 0; GeometryIdx < NumGeos; ++GeometryIdx)
				{
					const int32 VertexStart = VertexStarts[GeometryIdx];

					bool bAllIncluded = true;
					for (int32 Offset = 0; Offset < VertexCounts[GeometryIdx]; ++Offset)
					{
						const int32 VertexIdx = VertexStart + Offset;

						if (!InVertexSelection.Contains(VertexIdx))
						{
							bAllIncluded = false;
							break;
						}
					}

					if (bAllIncluded)
					{
						OutSelection.AddUnique(TransformIndices[GeometryIdx]);
					}
				}
			}
		}

		return OutSelection;
	}

	TArray<int32> FCollectionTransformSelectionFacade::ConvertFaceSelectionToTransformSelection(const TArray<int32>& InFaceSelection, bool bAllElementsMustBeSelected) const
	{
		TArray<int32> OutSelection;

		if (TransformIndexAttribute.IsValid() && FaceStartAttribute.IsValid() && FaceCountAttribute.IsValid())
		{
			const TManagedArray<int32>& TransformIndices = TransformIndexAttribute.Get();
			const TManagedArray<int32>& FaceStarts = FaceStartAttribute.Get();
			const TManagedArray<int32>& FaceCounts = FaceCountAttribute.Get();

			const int32 NumGeos = ConstCollection.NumElements(FGeometryCollection::GeometryGroup);

			if (!bAllElementsMustBeSelected)
			{
				for (int32 FaceIdx : InFaceSelection)
				{
					for (int32 GeometryIdx = 0; GeometryIdx < NumGeos; ++GeometryIdx)
					{
						const int32 FaceStart = FaceStarts[GeometryIdx];
						const int32 FaceEnd = FaceStart + FaceCounts[GeometryIdx];

						if (FaceIdx >= FaceStart && FaceIdx < FaceEnd)
						{
							OutSelection.AddUnique(TransformIndices[GeometryIdx]);
							break;
						}
					}
				}
			}
			else
			{
				for (int32 GeometryIdx = 0; GeometryIdx < NumGeos; ++GeometryIdx)
				{
					const int32 FaceStart = FaceStarts[GeometryIdx];

					bool bAllIncluded = true;
					for (int32 Offset = 0; Offset < FaceCounts[GeometryIdx]; ++Offset)
					{
						const int32 FaceIdx = FaceStart + Offset;

						if (!InFaceSelection.Contains(FaceIdx))
						{
							bAllIncluded = false;
							break;
						}
					}

					if (bAllIncluded)
					{
						OutSelection.AddUnique(TransformIndices[GeometryIdx]);
					}
				}
			}
		}

		return OutSelection;
	}

	TArray<int32> FCollectionTransformSelectionFacade::ConvertVertexSelectionToFaceSelection(const TArray<int32>& InVertexSelection, bool bAllElementsMustBeSelected) const
	{
		TArray<int32> OutSelection;

		if (FaceStartAttribute.IsValid() && FaceCountAttribute.IsValid() && IndicesAttribute.IsValid())
		{
			// Build VertexToFaceIndices inverse lookup
			const TManagedArray<int32>& FaceStarts = FaceStartAttribute.Get();
			const TManagedArray<int32>& FaceCounts = FaceCountAttribute.Get();

			const int32 NumFaces = ConstCollection.NumElements(FGeometryCollection::FacesGroup);
			const int32 NumVertices = ConstCollection.NumElements(FGeometryCollection::VerticesGroup);

			if (!bAllElementsMustBeSelected)
			{
				TArray<TArray<int32>> VertexToFaceIndices;
				VertexToFaceIndices.SetNum(NumVertices);

				const TManagedArray<FIntVector>& Indices = IndicesAttribute.Get();

				for (int32 FaceIdx = 0; FaceIdx < NumFaces; ++FaceIdx)
				{
					VertexToFaceIndices[Indices[FaceIdx].X].AddUnique(FaceIdx);
					VertexToFaceIndices[Indices[FaceIdx].Y].AddUnique(FaceIdx);
					VertexToFaceIndices[Indices[FaceIdx].Z].AddUnique(FaceIdx);
				}

				// Convert VertexSelection to FaceSelection
				for (int32 VertexIdx : InVertexSelection)
				{
					for (int32 FaceIdx = 0; FaceIdx < VertexToFaceIndices[VertexIdx].Num(); ++FaceIdx)
					{
						OutSelection.AddUnique(VertexToFaceIndices[VertexIdx][FaceIdx]);
					}
				}
			}
			else
			{
				const TManagedArray<FIntVector>& Indices = IndicesAttribute.Get();

				for (int32 FaceIdx = 0; FaceIdx < NumFaces; ++FaceIdx)
				{
					if (InVertexSelection.Contains(Indices[FaceIdx].X) &&
						InVertexSelection.Contains(Indices[FaceIdx].Y) &&
						InVertexSelection.Contains(Indices[FaceIdx].Z))
					{
						OutSelection.AddUnique(FaceIdx);
					}
				}
			}
		}

		return OutSelection;
	}

	void FCollectionTransformSelectionFacade::GetVertexToFaceIndicesArray(TArray<TArray<int32>>& VertexToFaceIndices) const
	{
		if (IndicesAttribute.IsValid())
		{
			// Build VertexToFaceIndices inverse lookup
			const int32 NumFaces = ConstCollection.NumElements(FGeometryCollection::FacesGroup);
			const int32 NumVertices = ConstCollection.NumElements(FGeometryCollection::VerticesGroup);

			VertexToFaceIndices.Empty();
			VertexToFaceIndices.SetNum(NumVertices);

			const TManagedArray<FIntVector>& Indices = IndicesAttribute.Get();

			for (int32 FaceIdx = 0; FaceIdx < NumFaces; ++FaceIdx)
			{
				if (Indices.IsValidIndex(FaceIdx))
				{
					VertexToFaceIndices[Indices[FaceIdx].X].AddUnique(FaceIdx);
					VertexToFaceIndices[Indices[FaceIdx].Y].AddUnique(FaceIdx);
					VertexToFaceIndices[Indices[FaceIdx].Z].AddUnique(FaceIdx);
				}
			}
		}
	}

	TArray<int32> FCollectionTransformSelectionFacade::ConvertTransformSelectionToFaceSelection(const TArray<int32>& InTransformSelection) const
	{
		TArray<int32> OutSelection;

		if (TransformToGeometryIndexAttribute.IsValid() && FaceStartAttribute.IsValid() && FaceCountAttribute.IsValid())
		{
			const TManagedArray<int32>& TransformToGeometryIndices = TransformToGeometryIndexAttribute.Get();
			const TManagedArray<int32>& FaceStarts = FaceStartAttribute.Get();
			const TManagedArray<int32>& FaceCounts = FaceCountAttribute.Get();

			for (int32 TransformIdx : InTransformSelection)
			{
				if (TransformToGeometryIndices.IsValidIndex(TransformIdx))
				{
					const int32 GeometryIdx = TransformToGeometryIndices[TransformIdx];
					if (FaceStarts.IsValidIndex(GeometryIdx))
					{
						const int32 FaceIdxStart = FaceStarts[GeometryIdx];

						for (int32 Offset = 0; Offset < FaceCounts[GeometryIdx]; ++Offset)
						{
							const int32 FaceIdx = FaceIdxStart + Offset;
							OutSelection.Add(FaceIdx);
						}
					}
				}
			}
		}

		return OutSelection;
	}

	TArray<int32> FCollectionTransformSelectionFacade::ConvertFaceSelectionToVertexSelection(const TArray<int32>& InFaceSelection) const
	{
		TArray<int32> OutSelection;

		if (IndicesAttribute.IsValid())
		{
			const TManagedArray<FIntVector>& Indices = IndicesAttribute.Get();

			for (int32 FaceIdx : InFaceSelection)
			{
				OutSelection.AddUnique(Indices[FaceIdx].X);
				OutSelection.AddUnique(Indices[FaceIdx].Y);
				OutSelection.AddUnique(Indices[FaceIdx].Z);
			}
		}

		return OutSelection;
	}

	TArray<int32> FCollectionTransformSelectionFacade::ConvertTransformSelectionToVertexSelection(const TArray<int32>& InTransformSelection) const
	{
		TArray<int32> OutSelection;

		if (TransformToGeometryIndexAttribute.IsValid() && VertexStartAttribute.IsValid() && VertexCountAttribute.IsValid())
		{
			const TManagedArray<int32>& TransformToGeometryIndices = TransformToGeometryIndexAttribute.Get();
			const TManagedArray<int32>& VertexStarts = VertexStartAttribute.Get();
			const TManagedArray<int32>& VertexCounts = VertexCountAttribute.Get();

			for (int32 TransformIdx : InTransformSelection)
			{
				if (TransformToGeometryIndices.IsValidIndex(TransformIdx))
				{
					const int32 GeometryIdx = TransformToGeometryIndices[TransformIdx];
					if (VertexStarts.IsValidIndex(GeometryIdx))
					{
						const int32 VertexIndexStart = VertexStarts[TransformToGeometryIndices[TransformIdx]];

						for (int32 Offset = 0; Offset < VertexCounts[TransformToGeometryIndices[TransformIdx]]; ++Offset)
						{
							const int32 VertexIdx = VertexIndexStart + Offset;
							OutSelection.Add(VertexIdx);
						}
					}
				}
			}
		}

		return OutSelection;
	}

	TArray<int32> FCollectionTransformSelectionFacade::ConvertTransformSelectionToGeometrySelection(const TArray<int32>& InTransformSelection) const
	{
		TArray<int32> OutSelection;

		if (TransformToGeometryIndexAttribute.IsValid())
		{
			const TManagedArray<int32>& TransformToGeometryIndices = TransformToGeometryIndexAttribute.Get();

			for (int32 TransformIdx : InTransformSelection)
			{
				if (TransformToGeometryIndices.IsValidIndex(TransformIdx))
				{
					const int32 GeometryIdx = TransformToGeometryIndices[TransformIdx];
					OutSelection.Add(GeometryIdx);
				}
			}
		}

		return OutSelection;
	}

	TArray<int32> FCollectionTransformSelectionFacade::ConvertGeometrySelectionToTransformSelection(const TArray<int32>& InGeometrySelection) const
	{
		TArray<int32> OutSelection;

		if (TransformIndexAttribute.IsValid())
		{
			const TManagedArray<int32>& TransformIndices = TransformIndexAttribute.Get();

			for (int32 GeometryIdx : InGeometrySelection)
			{
				if (TransformIndices.IsValidIndex(GeometryIdx))
				{
					const int32 TransformIdx = TransformIndexAttribute[GeometryIdx];
					OutSelection.Add(TransformIdx);
				}
			}
		}

		return OutSelection;
	}

	TArray<int32> FCollectionTransformSelectionFacade::ConvertGeometrySelectionToFaceSelection(const TArray<int32>& InGeometrySelection) const
	{
		TArray<int32> OutSelection;

		if (FaceStartAttribute.IsValid() && FaceCountAttribute.IsValid())
		{
			const TManagedArray<int32>& FaceStarts = FaceStartAttribute.Get();
			const TManagedArray<int32>& FaceCounts = FaceCountAttribute.Get();

			for (int32 GeometryIdx : InGeometrySelection)
			{
				if (FaceStarts.IsValidIndex(GeometryIdx))
				{
					const int32 FaceIdxStart = FaceStarts[GeometryIdx];

					for (int32 Offset = 0; Offset < FaceCounts[GeometryIdx]; ++Offset)
					{
						const int32 FaceIdx = FaceIdxStart + Offset;
						OutSelection.Add(FaceIdx);
					}
				}
			}
		}

		return OutSelection;
	}

	TArray<int32> FCollectionTransformSelectionFacade::ConvertGeometrySelectionToVertexSelection(const TArray<int32>& InGeometrySelection) const
	{
		TArray<int32> OutSelection;

		if (VertexStartAttribute.IsValid() && VertexCountAttribute.IsValid())
		{
			const TManagedArray<int32>& VertexStarts = VertexStartAttribute.Get();
			const TManagedArray<int32>& VertexCounts = VertexCountAttribute.Get();

			for (int32 GeometryIdx : InGeometrySelection)
			{
				if (VertexStarts.IsValidIndex(GeometryIdx))
				{
					const int32 VertexIdxStart = VertexStarts[GeometryIdx];

					for (int32 Offset = 0; Offset < VertexCounts[GeometryIdx]; ++Offset)
					{
						const int32 VertexIdx = VertexIdxStart + Offset;
						OutSelection.Add(VertexIdx);
					}
				}
			}
		}

		return OutSelection;
	}
	
	TArray<int32> FCollectionTransformSelectionFacade::ConvertFaceSelectionToGeometrySelection(const TArray<int32>& InFaceSelection, bool bAllElementsMustBeSelected) const
	{
		TArray<int32> OutSelection;

		if (FaceStartAttribute.IsValid() && FaceCountAttribute.IsValid())
		{
			const TManagedArray<int32>& FaceStarts = FaceStartAttribute.Get();
			const TManagedArray<int32>& FaceCounts = FaceCountAttribute.Get();

			const int32 NumGeos = ConstCollection.NumElements(FGeometryCollection::GeometryGroup);

			if (!bAllElementsMustBeSelected)
			{
				for (int32 FaceIdx : InFaceSelection)
				{
					for (int32 GeometryIdx = 0; GeometryIdx < NumGeos; ++GeometryIdx)
					{
						const int32 FaceStart = FaceStarts[GeometryIdx];
						const int32 FaceEnd = FaceStart + FaceCounts[GeometryIdx];

						if (FaceIdx >= FaceStart && FaceIdx < FaceEnd)
						{
							OutSelection.AddUnique(GeometryIdx);
							break;
						}
					}
				}
			}
			else
			{
				for (int32 GeometryIdx = 0; GeometryIdx < NumGeos; ++GeometryIdx)
				{
					const int32 FaceStart = FaceStarts[GeometryIdx];

					bool bAllIncluded = true;
					for (int32 Offset = 0; Offset < FaceCounts[GeometryIdx]; ++Offset)
					{
						const int32 FaceIdx = FaceStart + Offset;

						if (!InFaceSelection.Contains(FaceIdx))
						{
							bAllIncluded = false;
							break;
						}
					}

					if (bAllIncluded)
					{
						OutSelection.AddUnique(GeometryIdx);
					}
				}
			}
		}

		return OutSelection;
	}

	TArray<int32> FCollectionTransformSelectionFacade::ConvertVertexSelectionToGeometrySelection(const TArray<int32>& InVertexSelection, bool bAllElementsMustBeSelected) const
	{
		TArray<int32> OutSelection;

		if (VertexStartAttribute.IsValid() && VertexCountAttribute.IsValid())
		{
			const TManagedArray<int32>& VertexStarts = VertexStartAttribute.Get();
			const TManagedArray<int32>& VertexCounts = VertexCountAttribute.Get();

			const int32 NumGeos = ConstCollection.NumElements(FGeometryCollection::GeometryGroup);

			if (!bAllElementsMustBeSelected)
			{

				for (int32 VertexIdx : InVertexSelection)
				{
					for (int32 GeometryIdx = 0; GeometryIdx < NumGeos; ++GeometryIdx)
					{
						const int32 VertexStart = VertexStarts[GeometryIdx];
						const int32 VertexEnd = VertexStart + VertexCounts[GeometryIdx];

						if (VertexIdx >= VertexStart && VertexIdx < VertexEnd)
						{
							OutSelection.AddUnique(GeometryIdx);
							break;
						}
					}
				}
			}
			else
			{
				for (int32 GeometryIdx = 0; GeometryIdx < NumGeos; ++GeometryIdx)
				{
					const int32 VertexStart = VertexStarts[GeometryIdx];

					bool bAllIncluded = true;
					for (int32 Offset = 0; Offset < VertexCounts[GeometryIdx]; ++Offset)
					{
						const int32 VertexIdx = VertexStart + Offset;

						if (!InVertexSelection.Contains(VertexIdx))
						{
							bAllIncluded = false;
							break;
						}
					}

					if (bAllIncluded)
					{
						OutSelection.AddUnique(GeometryIdx);
					}
				}
			}
		}

		return OutSelection;
	}

	TArray<int32> FCollectionTransformSelectionFacade::ConvertTransformSelectionToCurveSelection(const TArray<int32>& InTransformSelection) const
	{
		TArray<int32> OutSelection;

		GeometryCollection::Facades::FCollectionCurveGeometryFacade CurveFacade(ConstCollection);
		if (TransformToGeometryIndexAttribute.IsValid() && CurveFacade.IsValid())
		{
			const TManagedArray<int32>& TransformToGeometryIndices = TransformToGeometryIndexAttribute.Get();
			const TArray<int32>& CurveOffsets = CurveFacade.GetGeometryCurveOffsets();

			for (int32 TransformIdx : InTransformSelection)
			{
				if (TransformToGeometryIndices.IsValidIndex(TransformIdx))
				{
					const int32 GeometryIdx = TransformToGeometryIndices[TransformIdx];
					if (CurveOffsets.IsValidIndex(GeometryIdx))
					{
						const int32 CurveIndexStart = GeometryIdx == 0 ? 0 : CurveOffsets[GeometryIdx - 1];
						const int32 CurveIndexEnd = CurveOffsets[GeometryIdx];

						for (int32 CurveIndex = CurveIndexStart; CurveIndex < CurveIndexEnd; ++CurveIndex)
						{
							OutSelection.Add(CurveIndex);
						}
					}
				}
			}
		}

		return OutSelection;
	}
	
	TArray<int32> FCollectionTransformSelectionFacade::ConvertGeometrySelectionToCurveSelection(const TArray<int32>& InGeometrySelection) const
	{
		TArray<int32> OutSelection;

		GeometryCollection::Facades::FCollectionCurveGeometryFacade CurveFacade(ConstCollection);
		if (CurveFacade.IsValid())
		{
			const TArray<int32>& CurveOffsets = CurveFacade.GetGeometryCurveOffsets();

			for (int32 GeometryIdx : InGeometrySelection)
			{
				if (CurveOffsets.IsValidIndex(GeometryIdx))
				{
					const int32 CurveIndexStart = GeometryIdx == 0 ? 0 : CurveOffsets[GeometryIdx - 1];
					const int32 CurveIndexEnd = CurveOffsets[GeometryIdx];

					for (int32 CurveIndex = CurveIndexStart; CurveIndex < CurveIndexEnd; ++CurveIndex)
					{
						OutSelection.Add(CurveIndex);
					}
				}
			}
		}
		return OutSelection;
	}
	
	TArray<int32> FCollectionTransformSelectionFacade::ConvertFaceSelectionToCurveSelection(const TArray<int32>& InFaceSelection, bool bAllElementsMustBeSelected) const
	{
		TArray<int32> OutSelection;

		GeometryCollection::Facades::FCollectionCurveGeometryFacade CurveFacade(ConstCollection);
		if (CurveFacade.IsValid())
		{
			const int32 NumCurves = CurveFacade.GetNumCurves();
			for (int32 CurveIdx = 0; CurveIdx < NumCurves; ++CurveIdx)
			{
				const int32 PointIndexStart = CurveIdx == 0 ? 0 :CurveFacade.GetCurvePointOffsets()[CurveIdx-1];
				const int32 PointIndexEnd = CurveFacade.GetCurvePointOffsets()[CurveIdx];

				const int32 FaceIndexStart = (PointIndexStart-CurveIdx) * 2;
				const int32 FaceIndexEnd = (PointIndexEnd-1-CurveIdx) * 2;

				bool bAllIncluded = true;
				for (int32 FaceIdx = FaceIndexStart; FaceIdx < FaceIndexEnd; ++FaceIdx)
				{
					if(InFaceSelection.Contains(FaceIdx))
					{
						if(!bAllElementsMustBeSelected)
						{
							OutSelection.Add(CurveIdx);
							break;
						}
					}
					else
					{
						bAllIncluded = false;
						if(bAllElementsMustBeSelected)
						{
							break;
						}
					}
				}
				if(bAllElementsMustBeSelected && bAllIncluded)
				{
					OutSelection.Add(CurveIdx);
				}
			}
		}

		return OutSelection;
	}

	TArray<int32> FCollectionTransformSelectionFacade::ConvertVertexSelectionToCurveSelection(const TArray<int32>& InVertexSelection, bool bAllElementsMustBeSelected) const
	{
		TArray<int32> OutSelectedCurves;

		GeometryCollection::Facades::FCollectionCurveGeometryFacade CurveFacade(ConstCollection);
		if (CurveFacade.IsValid())
		{
			const int32 NumCurves = CurveFacade.GetNumCurves();

			TBitArray<> SelectedCurves;
			SelectedCurves.Init(false, NumCurves);
			const TArray<int32>& CurveIndicesByPoint = CurveFacade.GetPointCurveIndices();

			for (const int32& VertexIndex : InVertexSelection)
			{
				// there's two vertices for each point
				const int32 PointIndex = (VertexIndex >> 1);
				if (CurveIndicesByPoint.IsValidIndex(PointIndex))
				{
					const int32 CurveIndex = CurveIndicesByPoint[PointIndex];
					if (SelectedCurves.IsValidIndex(CurveIndex) && !SelectedCurves[CurveIndex])
					{
						OutSelectedCurves.Add(CurveIndex);
						SelectedCurves[CurveIndex] = true;
					}
				}
			}

			// let's filter the selected curves further with a more expensive check if needed
			if (bAllElementsMustBeSelected)
			{
				// Convert vertex index array into an O(1) addressable storage
				TBitArray<> SelectedVertices;
				SelectedVertices.Init(false, CurveFacade.GetPointRestOrientations().Num() * 2);
				for (const int32& VertexIndex : InVertexSelection)
				{
					if (SelectedVertices.IsValidIndex(VertexIndex))
					{
						SelectedVertices[VertexIndex] = true;
					}
				}

				TArray<int32> PotentiallySelectedCurves = MoveTemp(OutSelectedCurves);
				OutSelectedCurves.Reserve(PotentiallySelectedCurves.Num());
				for (int32 CurveIdx : PotentiallySelectedCurves)
				{
					const int32 PointIndexStart = CurveIdx == 0 ? 0 : CurveFacade.GetCurvePointOffsets()[CurveIdx - 1];
					const int32 PointIndexEnd = CurveFacade.GetCurvePointOffsets()[CurveIdx];

					const int32 VertexIndexStart = PointIndexStart * 2;
					const int32 VertexIndexEnd = PointIndexEnd * 2;

					bool bAllIncluded = true;
					for (int32 VertexIdx = VertexIndexStart; VertexIdx < VertexIndexEnd; ++VertexIdx)
					{
						if (SelectedVertices.IsValidIndex(VertexIdx))
						{
							if (!SelectedVertices[VertexIdx])
							{
								bAllIncluded = false;
								break;
							}
						}
					}
					if (bAllIncluded)
					{
						OutSelectedCurves.Add(CurveIdx);
					}
				}
			}
		}
		return OutSelectedCurves;
	}

	void FCollectionTransformSelectionFacade::ConvertVertexSelectionToCurveSelection(const TBitArray<>& InVertexSelection, bool bAllElementsMustBeSelected, TBitArray<>& OutCurveSelection) const
	{
		GeometryCollection::Facades::FCollectionCurveGeometryFacade CurveFacade(ConstCollection);
		if (CurveFacade.IsValid())
		{
			const int32 NumCurves = CurveFacade.GetNumCurves();
			OutCurveSelection.Init(false, NumCurves);

			const TArray<int32>& CurveIndicesByPoint = CurveFacade.GetPointCurveIndices();

			const int32 NumVertices = InVertexSelection.Num();
			for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
			{
				if (InVertexSelection[VertexIndex])
				{
					// there's two vertices for each point
					const int32 PointIndex = (VertexIndex >> 1);
					if (CurveIndicesByPoint.IsValidIndex(PointIndex))
					{
						const int32 CurveIndex = CurveIndicesByPoint[PointIndex];
						if (OutCurveSelection.IsValidIndex(CurveIndex) && !OutCurveSelection[CurveIndex])
						{
							OutCurveSelection[CurveIndex] = true;
						}
					}
				}
			}

			// let's filter the selected curves further with a more expensive check if needed
			if (bAllElementsMustBeSelected)
			{
				for (int32 CurveIdx = 0; CurveIdx < NumCurves; ++CurveIdx)
				{
					const int32 PointIndexStart = CurveIdx == 0 ? 0 : CurveFacade.GetCurvePointOffsets()[CurveIdx - 1];
					const int32 PointIndexEnd = CurveFacade.GetCurvePointOffsets()[CurveIdx];

					const int32 VertexIndexStart = PointIndexStart * 2;
					const int32 VertexIndexEnd = PointIndexEnd * 2;

					for (int32 VertexIdx = VertexIndexStart; VertexIdx < VertexIndexEnd; ++VertexIdx)
					{
						if (InVertexSelection.IsValidIndex(VertexIdx))
						{
							if (!InVertexSelection[VertexIdx])
							{
								OutCurveSelection[CurveIdx] = false;
								break;
							}
						}
					}
				}
			}
		}
	}

	TArray<int32> FCollectionTransformSelectionFacade::ConvertCurveSelectionToVertexSelection(const TArray<int32>& InCurveSelection) const
	{
		TArray<int32> OutSelection;

		GeometryCollection::Facades::FCollectionCurveGeometryFacade CurveFacade(ConstCollection);
		if (CurveFacade.IsValid())
		{
			const int32 NumCurves = CurveFacade.GetNumCurves();
			for (const int32& CurveIdx : InCurveSelection)
			{
				const int32 PointIndexStart = CurveIdx == 0 ? 0 :CurveFacade.GetCurvePointOffsets()[CurveIdx-1];
				const int32 PointIndexEnd = CurveFacade.GetCurvePointOffsets()[CurveIdx];

				const int32 VertexIndexStart = PointIndexStart * 2;
				const int32 VertexIndexEnd = PointIndexEnd * 2;

				for (int32 VertexIdx = VertexIndexStart; VertexIdx < VertexIndexEnd; ++VertexIdx)
				{
					OutSelection.Add(VertexIdx);
				}
			}
		}

		return OutSelection;
	}

	TArray<int32> FCollectionTransformSelectionFacade::ConvertCurveSelectionToFaceSelection(const TArray<int32>& InCurveSelection) const
	{
		TArray<int32> OutSelection;

		GeometryCollection::Facades::FCollectionCurveGeometryFacade CurveFacade(ConstCollection);
		if (CurveFacade.IsValid())
		{
			const int32 NumCurves = CurveFacade.GetNumCurves();
			for (const int32& CurveIdx : InCurveSelection)
			{
				const int32 PointIndexStart = CurveIdx == 0 ? 0 :CurveFacade.GetCurvePointOffsets()[CurveIdx-1];
				const int32 PointIndexEnd = CurveFacade.GetCurvePointOffsets()[CurveIdx];

				const int32 FaceIndexStart = (PointIndexStart-CurveIdx) * 2;
				const int32 FaceIndexEnd = (PointIndexEnd-1-CurveIdx) * 2;
				
				for (int32 FaceIdx = FaceIndexStart; FaceIdx < FaceIndexEnd; ++FaceIdx)
				{
					OutSelection.Add(FaceIdx);
				}
			}
		}

		return OutSelection;
	}

	TArray<int32> FCollectionTransformSelectionFacade::ConvertCurveSelectionToGeometrySelection(const TArray<int32>& InCurveSelection, bool bAllElementsMustBeSelected) const
	{
		TArray<int32> OutSelection;

		GeometryCollection::Facades::FCollectionCurveGeometryFacade CurveFacade(ConstCollection);
		if (CurveFacade.IsValid())
		{
			const TArray<int32>& CurveOffsets = CurveFacade.GetGeometryCurveOffsets();
			
			for (int32 GeometryIdx = 0; GeometryIdx < CurveFacade.GetNumGeometry(); ++GeometryIdx)
			{
				if (CurveOffsets.IsValidIndex(GeometryIdx))
				{
					const int32 CurveIndexStart = GeometryIdx == 0 ? 0 : CurveOffsets[GeometryIdx - 1];
					const int32 CurveIndexEnd = CurveOffsets[GeometryIdx];

					bool bAllIncluded = true;
					for (int32 CurveIndex = CurveIndexStart; CurveIndex < CurveIndexEnd; ++CurveIndex)
					{
						if(InCurveSelection.Contains(CurveIndex))
						{
							if(!bAllElementsMustBeSelected)
							{
								OutSelection.Add(GeometryIdx);
								break;
							}
						}
						else
						{
							bAllIncluded = false;
							if(bAllElementsMustBeSelected)
							{
								break;
							}
						}
					}
					if(bAllElementsMustBeSelected && bAllIncluded)
					{
						OutSelection.Add(GeometryIdx);
					}
				}
			}
		}
		return OutSelection;
	}

	TArray<int32> FCollectionTransformSelectionFacade::ConvertCurveSelectionToTransformSelection(const TArray<int32>& InCurveSelection, bool bAllElementsMustBeSelected) const
	{
		TArray<int32> OutSelection;

		GeometryCollection::Facades::FCollectionCurveGeometryFacade CurveFacade(ConstCollection);
		if (CurveFacade.IsValid() && TransformIndexAttribute.IsValid())
		{
			const TManagedArray<int32>& TransformIndices = TransformIndexAttribute.Get();
			const TArray<int32>& CurveOffsets = CurveFacade.GetGeometryCurveOffsets();
			
			for (int32 GeometryIdx = 0; GeometryIdx < CurveFacade.GetNumGeometry(); ++GeometryIdx)
			{
				if (CurveOffsets.IsValidIndex(GeometryIdx))
				{
					const int32 CurveIndexStart = GeometryIdx == 0 ? 0 : CurveOffsets[GeometryIdx - 1];
					const int32 CurveIndexEnd = CurveOffsets[GeometryIdx];

					bool bAllIncluded = true;
					for (int32 CurveIndex = CurveIndexStart; CurveIndex < CurveIndexEnd; ++CurveIndex)
					{
						if(InCurveSelection.Contains(CurveIndex))
						{
							if(!bAllElementsMustBeSelected && TransformIndices.IsValidIndex(GeometryIdx))
							{
								OutSelection.Add(TransformIndices[GeometryIdx]);
								break;
							}
						}
						else
						{
							bAllIncluded = false;
							if(bAllElementsMustBeSelected)
							{
								break;
							}
						}
					}
					if(bAllElementsMustBeSelected && bAllIncluded && TransformIndices.IsValidIndex(GeometryIdx))
					{
						OutSelection.Add(TransformIndices[GeometryIdx]);
					}
				}
			}
		}
		return OutSelection;
	}
}; // GeometryCollection::Facades
