// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GeometryCollection.cpp: FGeometryCollection methods.
=============================================================================*/

#include "GeometryCollection/Facades/CollectionTransformSourceFacade.h"
#include "GeometryCollection/TransformCollection.h"
#include "GeometryCollection/ManagedArrayAccessor.h"

namespace GeometryCollection::Facades
{
	// Groups 
	const FName FTransformSource::TransformSourceGroupName = "TransformSource";

	// Attributes
	const FName FTransformSource::SourceNameAttributeName = "Name";
	const FName FTransformSource::SourceGuidAttributeName = "GuidID";
	const FName FTransformSource::SourceRootsAttributeName = "Roots";
	const FName FTransformSource::SourceSkeletalMeshNameAttributeName = "SkeletalMeshName";

	FTransformSource::FTransformSource(FManagedArrayCollection& InCollection)
		: SourceNameAttribute(InCollection, SourceNameAttributeName, TransformSourceGroupName)
		, SourceGuidAttribute(InCollection, SourceGuidAttributeName, TransformSourceGroupName)
		, SourceRootsAttribute(InCollection, SourceRootsAttributeName, TransformSourceGroupName, FTransformCollection::TransformGroup)
		, SourceSkeletalMeshNameAttribute(InCollection, SourceSkeletalMeshNameAttributeName, TransformSourceGroupName)
	{}

	FTransformSource::FTransformSource(const FManagedArrayCollection& InCollection)
		: SourceNameAttribute(InCollection, SourceNameAttributeName, TransformSourceGroupName)
		, SourceGuidAttribute(InCollection, SourceGuidAttributeName, TransformSourceGroupName)
		, SourceRootsAttribute(InCollection, SourceRootsAttributeName, TransformSourceGroupName, FTransformCollection::TransformGroup)
		, SourceSkeletalMeshNameAttribute(InCollection, SourceSkeletalMeshNameAttributeName, TransformSourceGroupName)
	{}
	//
	//  Initialization
	//

	void FTransformSource::DefineSchema()
	{
		check(!IsConst());
		SourceNameAttribute.Add();
		SourceGuidAttribute.Add();
		SourceRootsAttribute.Add();
		SourceSkeletalMeshNameAttribute.Add();
	}

	bool FTransformSource::IsValid() const
	{
		return SourceNameAttribute.IsValid() && SourceGuidAttribute.IsValid() && SourceRootsAttribute.IsValid(); 
	}

	//
	//  Add Data
	//
	void FTransformSource::AddTransformSource(const FString& InName, const FString& InGuid, const TSet<int32>& InRoots, const FString& SKMName)
	{
		check(!IsConst());
		DefineSchema();

		int Idx = SourceNameAttribute.AddElements(1);
		SourceNameAttribute.Modify()[Idx] = InName;
		SourceGuidAttribute.Modify()[Idx] = InGuid;
		SourceRootsAttribute.Modify()[Idx] = InRoots;
		SourceSkeletalMeshNameAttribute.Modify()[Idx] = SKMName;
	}

	//
	//  Get Data
	//
	TSet<int32> FTransformSource::GetTransformSource(const FString& InName, const FString& InGuid, const FString& SKMName) const
	{
		if (IsValid())
		{
			const TManagedArray<TSet<int32>>& Roots = SourceRootsAttribute.Get();
			const TManagedArray<FString>& Guids = SourceGuidAttribute.Get();
			const TManagedArray<FString>&Names = SourceNameAttribute.Get();

			int32 GroupNum = SourceNameAttribute.Num();
			for (int i = 0; i < GroupNum; i++)
			{
				if (Guids[i] == InGuid && Names[i].Equals(InName))
				{
					if (!SourceSkeletalMeshNameAttribute.IsValid() || SourceSkeletalMeshNameAttribute[i].IsEmpty() || SourceSkeletalMeshNameAttribute[i].Equals(SKMName))
					{
						return Roots[i];
					}
				}
			}
		}
		return TSet<int32>();
	}
};


