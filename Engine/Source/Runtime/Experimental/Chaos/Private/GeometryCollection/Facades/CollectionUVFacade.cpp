// Copyright Epic Games, Inc. All Rights Reserved.


#include "GeometryCollection/Facades/CollectionUVFacade.h"
#include "GeometryCollection/TransformCollection.h"
#include "GeometryCollection/GeometryCollection.h"

namespace GeometryCollection::UV
{
	const FName UVLayerNames[GeometryCollectionUV::MAX_NUM_UV_CHANNELS] = {
		"UVLayer0",
		"UVLayer1",
		"UVLayer2",
		"UVLayer3",
		"UVLayer4",
		"UVLayer5",
		"UVLayer6",
		"UVLayer7"
	};

	const FName VerticesGroupName = "Vertices";

	bool HasValidUVs(const FManagedArrayCollection& Collection)
	{
		return Collection.HasAttribute(UVLayerNames[0], VerticesGroupName);
	}

	void DefineUVSchema(FManagedArrayCollection& Collection)
	{
		// Only the first layer's attribute must always be present
		Collection.AddAttribute<FVector2f>(UVLayerNames[0], VerticesGroupName);
	}

	bool SetNumUVLayers(FManagedArrayCollection& Collection, int32 NumLayers)
	{
		if (NumLayers < 1 || NumLayers > GeometryCollectionUV::MAX_NUM_UV_CHANNELS)
		{
			return false;
		}

		// enable layers 0 to Num
		for (int32 LayerIdx = 0; LayerIdx < NumLayers; ++LayerIdx)
		{
			Collection.AddAttribute<FVector2f>(UVLayerNames[LayerIdx], VerticesGroupName); // Note: AddAttribute checks HasAttribute; no need to do so here
		}
		// disable layers Num to Max
		for (int32 LayerIdx = NumLayers; LayerIdx < GeometryCollectionUV::MAX_NUM_UV_CHANNELS; ++LayerIdx)
		{
			Collection.RemoveAttribute(UVLayerNames[LayerIdx], VerticesGroupName);
		}
		return true;
	}

	int32 GetNumUVLayers(const FManagedArrayCollection& Collection)
	{
		// Any collection with valid UV layers must always have layer 0
		checkSlow(Collection.HasAttribute(UVLayerNames[0], VerticesGroupName));

		int32 LayerCount = 1;
		for (; LayerCount < GeometryCollectionUV::MAX_NUM_UV_CHANNELS; ++LayerCount)
		{
			if (!Collection.HasAttribute(UVLayerNames[LayerCount], VerticesGroupName))
			{
				break;
			}
		}

		return LayerCount;
	}
}
