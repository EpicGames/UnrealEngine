// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GeometryCollection.cpp: FGeometryCollection methods.
=============================================================================*/

#include "GeometryCollection/Facades/CollectionVertexBoneWeightsFacade.h"

#include "CoreMinimal.h"
#include "MeshAttributes.h"
#include "GeometryCollection/Facades/CollectionKinematicBindingFacade.h"
#include "GeometryCollection/GeometryCollection.h"

DEFINE_LOG_CATEGORY_STATIC(LogDataflowVertexBoneWeightsFacade, Log, All);

namespace GeometryCollection::Facades
{

	// Attributes
	const FName FVertexBoneWeightsFacade::BoneWeightsAttributeName = "BoneWeights";
	const FName FVertexBoneWeightsFacade::BoneIndicesAttributeName = "BoneIndices";
	const FName FVertexBoneWeightsFacade::KinematicWeightAttributeName = "KinematicWeight";
	const FName FVertexBoneWeightsFacade::SkeletalMeshAttributeName = "SkeletalMesh";
	const FName FVertexBoneWeightsFacade::GeometryLODAttributeName = "GeometryLOD";

	// Deprecated
	const FName FVertexBoneWeightsFacade::DeprecatedBoneIndicesAttributeName = "BoneWeightsIndex";
	const FName FVertexBoneWeightsFacade::DeprecatedKinematicFlagAttributeName = "Kinematic";
	
	FVertexBoneWeightsFacade::FVertexBoneWeightsFacade(FManagedArrayCollection& InCollection, const bool bInInternalWeights)
		: ConstCollection(InCollection)
		, Collection(&InCollection)
		, BoneIndicesAttribute(InCollection, BoneIndicesAttributeName, FGeometryCollection::VerticesGroup, bInInternalWeights ? FTransformCollection::TransformGroup : NAME_None)
		, BoneWeightsAttribute(InCollection, BoneWeightsAttributeName, FGeometryCollection::VerticesGroup, bInInternalWeights ? FTransformCollection::TransformGroup : NAME_None)
		, KinematicWeightAttribute(InCollection, KinematicWeightAttributeName, FGeometryCollection::VerticesGroup)
		, GeometryLODAttribute(InCollection, GeometryLODAttributeName, FGeometryCollection::GeometryGroup)
		, SkeletalMeshAttribute(InCollection, SkeletalMeshAttributeName, FGeometryCollection::GeometryGroup)
		, bInternalWeights(bInInternalWeights)
	{
		DefineSchema();
	}

	FVertexBoneWeightsFacade::FVertexBoneWeightsFacade(const FManagedArrayCollection& InCollection, const bool bInInternalWeights)
		: ConstCollection(InCollection)
		, Collection(nullptr)
		, BoneIndicesAttribute(InCollection, BoneIndicesAttributeName, FGeometryCollection::VerticesGroup, bInInternalWeights ? FTransformCollection::TransformGroup : NAME_None)
		, BoneWeightsAttribute(InCollection, BoneWeightsAttributeName, FGeometryCollection::VerticesGroup, bInInternalWeights ? FTransformCollection::TransformGroup : NAME_None)
		, KinematicWeightAttribute(InCollection, KinematicWeightAttributeName, FGeometryCollection::VerticesGroup)
		, GeometryLODAttribute(InCollection, GeometryLODAttributeName, FGeometryCollection::GeometryGroup)
		, SkeletalMeshAttribute(InCollection, SkeletalMeshAttributeName, FGeometryCollection::GeometryGroup)
		, bInternalWeights(bInInternalWeights)
	{
	}

	//
	//  Initialization
	//

	void FVertexBoneWeightsFacade::DefineSchema()
	{
		check(!IsConst());
		BoneIndicesAttribute.Add();
		BoneWeightsAttribute.Add();
		KinematicWeightAttribute.AddAndFill(0.0f);

		// SkeletalMesh is Dataflow editor-time scratch and must not persist: it holds
		// TObjectPtr<UObject>, which a base FArchive (e.g. the DDC cooker's FMemoryWriter)
		// cannot serialize. Drop any legacy persistent entry first so the re-Add lands
		// with bSaved=false.
		if (SkeletalMeshAttribute.IsPersistent())
		{
			SkeletalMeshAttribute.Remove();
		}
		SkeletalMeshAttribute.Add(ManageArrayAccessor::EPersistencePolicy::KeepExistingPersistence);

		GeometryLODAttribute.Add();
		
		if (!BoneIndicesAttribute.IsValid())
		{
			UE_LOGF(LogDataflowVertexBoneWeightsFacade, Warning, "FVertexBoneWeightsFacade failed to initialize because '%ls' attribute from '%ls' group expected type TArray<int32>.", *FVertexBoneWeightsFacade::BoneIndicesAttributeName.ToString(), *FGeometryCollection::VerticesGroup.ToString());
		}
		if (!BoneWeightsAttribute.IsValid())
		{
			UE_LOGF(LogDataflowVertexBoneWeightsFacade, Warning, "FVertexBoneWeightsFacade failed to initialize because '%ls' attribute from '%ls' group expected type TArray<float>.", *FVertexBoneWeightsFacade::BoneWeightsAttributeName.ToString(), *FGeometryCollection::VerticesGroup.ToString());
		}
		if (!KinematicWeightAttribute.IsValid())
		{
			UE_LOGF(LogDataflowVertexBoneWeightsFacade, Warning, "FVertexBoneWeightsFacade failed to initialize because '%ls' attribute from '%ls' group expected type <float>.", *FVertexBoneWeightsFacade::KinematicWeightAttributeName.ToString(), *FGeometryCollection::VerticesGroup.ToString());
		}
		if (!SkeletalMeshAttribute.IsValid())
		{
			UE_LOGF(LogDataflowVertexBoneWeightsFacade, Warning, "FVertexBoneWeightsFacade failed to initialize because '%ls' attribute from '%ls' group expected type <TObjectPtr<UObject>>.", *FVertexBoneWeightsFacade::SkeletalMeshAttributeName.ToString(), *FGeometryCollection::GeometryGroup.ToString());
		}
		if (!GeometryLODAttribute.IsValid())
		{
			UE_LOGF(LogDataflowVertexBoneWeightsFacade, Warning, "FVertexBoneWeightsFacade failed to initialize because '%ls' attribute from '%ls' group expected type <int32>.", *FVertexBoneWeightsFacade::GeometryLODAttributeName.ToString(), *FGeometryCollection::GeometryGroup.ToString());
		}
	}

	bool FVertexBoneWeightsFacade::IsValid() const
	{
		return BoneIndicesAttribute.IsValid() && BoneWeightsAttribute.IsValid() && KinematicWeightAttribute.IsValid() && SkeletalMeshAttribute.IsValid() && GeometryLODAttribute.IsValid();
	}

	bool FVertexBoneWeightsFacade::HasValidBoneIndicesAndWeights() const
	{
		return (ConstCollection.HasAttribute(DeprecatedBoneIndicesAttributeName, FGeometryCollection::VerticesGroup) 
			|| BoneIndicesAttribute.IsValid()) && BoneWeightsAttribute.IsValid();
	}

	void FVertexBoneWeightsFacade::ModifyGeometryBinding(const int32 GeometryIndex, const TObjectPtr<UObject>& SkeletalMesh, const int32 GeometryLOD)
	{
		TManagedArray< TObjectPtr<UObject> >& SkeletalMeshes = SkeletalMeshAttribute.Modify();
		TManagedArray< int32 >& GeometryLODs = GeometryLODAttribute.Modify();

		if(0 <= GeometryIndex && GeometryIndex < ConstCollection.NumElements(FGeometryCollection::GeometryGroup))
		{
			SkeletalMeshes[GeometryIndex] = SkeletalMesh;
			GeometryLODs[GeometryIndex] = GeometryLOD;
		}
	}

	//
	//  Add Weights from a bone to a vertex 
	//
	void FVertexBoneWeightsFacade::AddBoneWeight(int32 VertexIndex, int32 BoneIndex, float BoneWeight)
	{
		TManagedArray< TArray<int32> >& IndicesArray = BoneIndicesAttribute.Modify();
		TManagedArray< TArray<float> >& WeightsArray = BoneWeightsAttribute.Modify();

		if (0 <= VertexIndex && VertexIndex < NumVertices())
		{
			if (!bInternalWeights || (0 <= BoneIndex && BoneIndex < NumBones()))
			{
				IndicesArray[VertexIndex].Add(BoneIndex);
				WeightsArray[VertexIndex].Add(BoneWeight);
			}
		}
	}

	void FVertexBoneWeightsFacade::ModifyKinematicWeight(int32 VertexIndex, const float KinematicWeight)
	{
		if (KinematicWeightAttribute.IsValid() && KinematicWeightAttribute.IsValidIndex(VertexIndex))
		{
			KinematicWeightAttribute.ModifyAt(VertexIndex, KinematicWeight);
		}
	}

	void FVertexBoneWeightsFacade::ModifyBoneWeight(int32 VertexIndex, const TArray<int32>& VertexBoneIndices, const TArray<float>& VertexBoneWeights)
	{
		if(VertexBoneIndices.Num() == VertexBoneWeights.Num())
		{ 
			TManagedArray< TArray<int32> >& IndicesArray = BoneIndicesAttribute.Modify();
			TManagedArray< TArray<float> >& WeightsArray = BoneWeightsAttribute.Modify();

			if (IndicesArray.IsValidIndex(VertexIndex) && WeightsArray.IsValidIndex(VertexIndex))
			{
				IndicesArray[VertexIndex].Empty();
				WeightsArray[VertexIndex].Empty();
				float TotalWeight = 0.f;
				int32 LocalIndex = 0;

				for (const int32& BoneIndex : VertexBoneIndices)
				{
					if (!bInternalWeights || (0 <= BoneIndex && BoneIndex < NumBones()))
					{
						const float BoneWeight = VertexBoneWeights[LocalIndex++];

						IndicesArray[VertexIndex].Add(BoneIndex);
						WeightsArray[VertexIndex].Add(BoneWeight);
						TotalWeight += BoneWeight;
					}
				}
				if (TotalWeight < 1.f - UE_KINDA_SMALL_NUMBER || TotalWeight > 1.f + UE_KINDA_SMALL_NUMBER)
				{
					UE_LOGF(LogChaos, Warning, "FVertexBoneWeightsFacade::ModifyBoneWeight: Bone weight sum %f is not 1 on vertex %d", TotalWeight, VertexIndex);
				}
			}
		}
	}

	void FVertexBoneWeightsFacade::SetVertexKinematic(int32 VertexIndex, bool Value)
	{
		if (KinematicWeightAttribute.IsValid() && KinematicWeightAttribute.IsValidIndex(VertexIndex))
		{
			KinematicWeightAttribute.ModifyAt(VertexIndex, Value);
		}
	}

	void FVertexBoneWeightsFacade::SetVertexArrayKinematic(const TArray<int32>& VertexIndices, bool Value)
	{
		if (KinematicWeightAttribute.IsValid())
		{
			TManagedArray<float>& KinematicWeights = KinematicWeightAttribute.Modify();
			for (const int32& VertexIndex : VertexIndices)
			{
				if (KinematicWeights.IsValidIndex(VertexIndex))
				{
					KinematicWeights[VertexIndex] = Value;
				}
			}
		}
	}

	bool FVertexBoneWeightsFacade::IsKinematicVertex(int32 VertexIndex) const
	{
		if(ConstCollection.HasAttribute(DeprecatedKinematicFlagAttributeName, FGeometryCollection::VerticesGroup))
    	{
    		const TManagedArray<bool>& KinematicFlag = ConstCollection.GetAttribute<bool>(DeprecatedKinematicFlagAttributeName, FGeometryCollection::VerticesGroup);
    		return KinematicFlag.IsValidIndex(VertexIndex) && KinematicFlag.GetConstArray()[VertexIndex];
    	}
		else if (KinematicWeightAttribute.IsValid())
		{
			return KinematicWeightAttribute.IsValidIndex(VertexIndex) && (KinematicWeightAttribute.Get()[VertexIndex] == 1.0);
		}
		else //backward compatibility for KinematicAttribute added in 5.5
		{
			return BoneIndicesAttribute.IsValid() && BoneIndicesAttribute.IsValidIndex(VertexIndex) && BoneIndicesAttribute.Get()[VertexIndex].Num()
				&& BoneWeightsAttribute.IsValid() && BoneWeightsAttribute.IsValidIndex(VertexIndex) && BoneWeightsAttribute.Get()[VertexIndex].Num();
		}
	};

	const TManagedArray< TArray<int32> >* FVertexBoneWeightsFacade::FindBoneIndices()  const
	{
		if(ConstCollection.HasAttribute(DeprecatedBoneIndicesAttributeName, FGeometryCollection::VerticesGroup))
		{
			return ConstCollection.FindAttribute<TArray<int32> >(DeprecatedBoneIndicesAttributeName, FGeometryCollection::VerticesGroup);
		}
		else
		{
			return BoneIndicesAttribute.Find();
		}
	}
	
	const TManagedArray< TArray<int32> >& FVertexBoneWeightsFacade::GetBoneIndices() const
	{
		if(ConstCollection.HasAttribute(DeprecatedBoneIndicesAttributeName, FGeometryCollection::VerticesGroup))
		{
			return ConstCollection.GetAttribute<TArray<int32> >(DeprecatedBoneIndicesAttributeName, FGeometryCollection::VerticesGroup);
		}
		else
		{
			return BoneIndicesAttribute.Get();
		}
	}

	const TManagedArray< TArray<float> >* FVertexBoneWeightsFacade::FindBoneWeights()  const
	{
		return BoneWeightsAttribute.Find();
	}
	
	const TManagedArray< TArray<float> >& FVertexBoneWeightsFacade::GetBoneWeights() const
	{
		return BoneWeightsAttribute.Get();
	}

	//
	//  Add Weights from Selection 
	//

	void FVertexBoneWeightsFacade::NormalizeBoneWeights()
	{
		if (IsValid())
		{
			TManagedArray< TArray<float> >& WeightsArray = BoneWeightsAttribute.Modify();
			for(TArray<float>& BoneWeights : WeightsArray)
			{
				float TotalWeight = 0.0;
				for(float& BoneWeight : BoneWeights)
				{
					TotalWeight += BoneWeight;
				}
				if (TotalWeight > UE_KINDA_SMALL_NUMBER)
				{
					for(float& BoneWeight : BoneWeights)
					{
						BoneWeight /= TotalWeight;
					}
				}
			}
		}
	}

	void FVertexBoneWeightsFacade::AddBoneWeightsFromKinematicBindings()
	{
		check(!IsConst());
		DefineSchema();

		if (IsValid() && bInternalWeights)
		{
			TManagedArray< TArray<int32> >& IndicesArray = BoneIndicesAttribute.Modify();
			TManagedArray< TArray<float> >& WeightsArray = BoneWeightsAttribute.Modify();

			TArray<float> TotalWeights;
			TotalWeights.Init(0.f, WeightsArray.Num());

			int32 VertexIndex = 0;
			for(TArray<float>& BoneWeights : WeightsArray)
			{
				for(float& BoneWeight : BoneWeights)
				{
					TotalWeights[VertexIndex] += BoneWeight;
				}
				++VertexIndex;
			}

			GeometryCollection::Facades::FKinematicBindingFacade BindingFacade(ConstCollection);
			for (int32 Kdx = BindingFacade.NumKinematicBindings() - 1; 0 <= Kdx; Kdx--)
			{
				int32 BoneIndex;
				TArray<int32> VertexIndices;
				TArray<float> VertexWeights;

				BindingFacade.GetBoneBindings(BindingFacade.GetKinematicBindingKey(Kdx), BoneIndex, VertexIndices, VertexWeights);

				if (0 <= BoneIndex && BoneIndex < NumBones())
				{
					for (int32 LocalVertex = 0; LocalVertex < VertexIndices.Num(); ++LocalVertex)
					{
						VertexIndex = VertexIndices[LocalVertex];
						const float VertexWeight = VertexWeights[LocalVertex];
						if (0 <= VertexIndex && VertexIndex < NumVertices() && !IndicesArray[VertexIndex].Contains(BoneIndex))
						{
							SetVertexKinematic(VertexIndex);
							int32 LocalBone = IndicesArray[VertexIndex].Find(BoneIndex);
							if (TotalWeights[VertexIndex] + VertexWeight <= 1.f + UE_KINDA_SMALL_NUMBER)
							{
								IndicesArray[VertexIndex].Add(BoneIndex);
								WeightsArray[VertexIndex].Add(VertexWeight);

								// Maybe we should re-normalize the weights after the kinematic bindings loop
								TotalWeights[VertexIndex] += VertexWeight;
							}
							else
							{
								UE_LOGF(LogChaos, Warning, "Bone weight sum %f exceeds 1 on vertex %d", TotalWeights[VertexIndex] + VertexWeight, VertexIndex);
							}
						}
					}
				}
			}
			NormalizeBoneWeights();
		}
	}
}


