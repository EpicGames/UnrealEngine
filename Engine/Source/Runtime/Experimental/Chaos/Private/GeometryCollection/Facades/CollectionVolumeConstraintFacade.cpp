// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	FleshCollection.cpp: FFleshCollection methods.
=============================================================================*/

#include "GeometryCollection/Facades/CollectionVolumeConstraintFacade.h"

namespace GeometryCollection::Facades
{
	// Attributes
	const FName FVolumeConstraintFacade::GroupName("VolumeConstraints");
	const FName FVolumeConstraintFacade::VolumeIndex("VolumeIndex");
	const FName FVolumeConstraintFacade::Stiffness("Stiffness");

	FVolumeConstraintFacade::FVolumeConstraintFacade(FManagedArrayCollection& InCollection)
		: ConstCollection(InCollection)
		, Collection(&InCollection)
		, VolumeIndexAttribute(InCollection, VolumeIndex, GroupName, FGeometryCollection::VerticesGroup)
		, StiffnessAttribute(InCollection, Stiffness, GroupName)
	{
		DefineSchema();
	}

	FVolumeConstraintFacade::FVolumeConstraintFacade(const FManagedArrayCollection& InCollection)
		: ConstCollection(InCollection)
		, Collection(nullptr)
		, VolumeIndexAttribute(InCollection, VolumeIndex, GroupName, FGeometryCollection::VerticesGroup)
		, StiffnessAttribute(InCollection, Stiffness, GroupName)
	{}

	bool FVolumeConstraintFacade::IsValid() const
	{
		return VolumeIndexAttribute.IsValid() && StiffnessAttribute.IsValid();
	}

	void FVolumeConstraintFacade::DefineSchema()
	{
		check(!IsConst());
		VolumeIndexAttribute.Add();
		StiffnessAttribute.Add();
	}


	int32 FVolumeConstraintFacade::AddVolumeConstraint(const FIntVector4& NewVolumeIndex, float NewStiffness)
	{
		check(!IsConst());
		if (IsValid())
		{
			int32 NewIndex = VolumeIndexAttribute.AddElements(1);
			VolumeIndexAttribute.Modify()[NewIndex] = NewVolumeIndex;
			StiffnessAttribute.Modify()[NewIndex] = NewStiffness;
			return NewIndex;
		}
		return INDEX_NONE;
	}

	FIntVector4 FVolumeConstraintFacade::GetVolumeIndex(int32 AttributeIndex) const
	{
		if (IsValid() && VolumeIndexAttribute.IsValidIndex(AttributeIndex))
		{
			return VolumeIndexAttribute[AttributeIndex];
		}
		return FIntVector4(INDEX_NONE);
	}

	float FVolumeConstraintFacade::GetStiffness(int32 AttributeIndex) const
	{
		if (IsValid() && StiffnessAttribute.IsValidIndex(AttributeIndex))
		{
			return StiffnessAttribute[AttributeIndex];
		}
		return 0.f;
	}

	int32 FVolumeConstraintFacade::RemoveInvalidVolumeConstraint()
	{
		check(!IsConst());
		TArray<int32> InvalidConstraintIndices;
		if (IsValid())
		{
			const int32 NumElemDependency = Collection->NumElements(VolumeIndexAttribute.GetGroupDependency());
			for (int32 Index = 0; Index < VolumeIndexAttribute.Num(); ++Index)
			{
				for (int32 LocalIdx = 0; LocalIdx < 4; ++LocalIdx)
				{
					if (VolumeIndexAttribute[Index][LocalIdx] < 0 || VolumeIndexAttribute[Index][LocalIdx] > NumElemDependency)
					{
						InvalidConstraintIndices.Add(Index);
						break;
					}
				}
			}
			Collection->RemoveElements(GroupName, InvalidConstraintIndices);
		}
		return InvalidConstraintIndices.Num();
	}

	int32 FVolumeConstraintFacade::RemoveVolumeConstraintBetween(TFunctionRef<bool(const int32 VertexIdx)> IsVertexGroup1, TFunctionRef<bool(const int32 VertexIdx)> IsVertexGroup2)
	{
		check(!IsConst());
		TArray<int32> ConstraintIndicesToRemove;
		if (IsValid())
		{
			for (int32 Index = 0; Index < Collection->NumElements(GroupName); ++Index)
			{
				bool HasGroup1 = false;
				bool HasGroup2 = false;
				for (int32 LocalIdx = 0; LocalIdx < 4; ++LocalIdx)
				{
					int32 VertIdx = VolumeIndexAttribute[Index][LocalIdx];
					HasGroup1 = HasGroup1 || IsVertexGroup1(VertIdx);
					HasGroup2 = HasGroup2 || IsVertexGroup2(VertIdx);	
				}
				if (HasGroup1 && HasGroup2)
				{
					ConstraintIndicesToRemove.Add(Index);
				}
			}
			Collection->RemoveElements(GroupName, ConstraintIndicesToRemove);
		}
		return ConstraintIndicesToRemove.Num();
	}
}
