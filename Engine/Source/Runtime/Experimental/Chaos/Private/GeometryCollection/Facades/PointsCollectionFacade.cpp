// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/Facades/PointsCollectionFacade.h"
#include "GeometryCollection/GeometryCollection.h"

namespace GeometryCollection::Facades
{
	FPointsCollectionFacade::FPointsCollectionFacade(FManagedArrayCollection& InCollection)
		: ConstCollection(InCollection)
		, Collection(&InCollection) 
	{}

	FPointsCollectionFacade::FPointsCollectionFacade(const FManagedArrayCollection& InCollection)
		: ConstCollection(InCollection)
		, Collection(nullptr)
	{}

	//
	//  Add points
	//
	void FPointsCollectionFacade::AddPoints(const TArray<FVector>& InPoints)
	{
		const int32 NumPoints = InPoints.Num();
		const int32 NumTransforms = Collection->NumElements(FGeometryCollection::TransformGroup);
		const int32 NumGeometry = Collection->NumElements(FGeometryCollection::GeometryGroup);
		const int32 NumVertices = Collection->NumElements(FGeometryCollection::VerticesGroup);

		Collection->AddElements(1, FGeometryCollection::TransformGroup);
		Collection->AddElements(1, FGeometryCollection::GeometryGroup);
		Collection->AddElements(NumPoints, FGeometryCollection::VerticesGroup);

		TManagedArray<FTransform>& Transform = Collection->AddAttribute<FTransform>("Transform", FGeometryCollection::TransformGroup);
		TManagedArray<FString>& BoneName = Collection->AddAttribute<FString>("BoneName", FGeometryCollection::TransformGroup);
		TManagedArray<FLinearColor>& BoneColor = Collection->AddAttribute<FLinearColor>("BoneColor", FGeometryCollection::TransformGroup);
		TManagedArray<int32>& Parent = Collection->AddAttribute<int32>("Parent", FGeometryCollection::TransformGroup);
		TManagedArray<TSet<int32>>& Children = Collection->AddAttribute<TSet<int32>>("Children", FGeometryCollection::TransformGroup);
		TManagedArray<int32>& TransformToGeometryIndex = Collection->AddAttribute<int32>("TransformToGoemetryIndex", FGeometryCollection::TransformGroup);
		TManagedArray<int32>& BoneMap = Collection->AddAttribute<int32>("BoneMap", FGeometryCollection::VerticesGroup);
		TManagedArray<int32>& TransformIndex = Collection->AddAttribute<int32>("TransformIndex", FGeometryCollection::GeometryGroup);
		TManagedArray<int32>& VertexStart = Collection->AddAttribute<int32>("VertexStart", FGeometryCollection::GeometryGroup);
		TManagedArray<int32>& VertexCount = Collection->AddAttribute<int32>("VertexCount", FGeometryCollection::GeometryGroup);
		TManagedArray<FVector3f>& Vertex = Collection->AddAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup);

		Transform[NumTransforms] = FTransform::Identity;
		BoneName[NumTransforms] = FString("Points");
		BoneColor[NumTransforms] = FLinearColor(0.02f, 0.01f, 0.1f, 1.0f);
		Parent[NumTransforms] = -1;
		Children[NumTransforms] = TSet<int32>();
		TransformToGeometryIndex[NumTransforms] = NumGeometry;
		TransformIndex[NumGeometry] = NumTransforms;
		VertexStart[NumGeometry] = NumVertices;
		VertexCount[NumGeometry] = NumPoints;
	}

	void FPointsCollectionFacade::AddPointsWithFloatAttribute(const TArray<FVector>& InPoints, const FName InAttributeName, const TArray<float>& InValues)
	{
		const int32 NumPoints = InPoints.Num();
		const int32 NumVertices = Collection->NumElements(FGeometryCollection::VerticesGroup);

		AddPoints(InPoints);

		TManagedArray<int32>& BoneMap = Collection->AddAttribute<int32>("BoneMap", FGeometryCollection::VerticesGroup);
		TManagedArray<FVector3f>& Vertex = Collection->AddAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup);

		TManagedArray<float>& AttrArr = Collection->AddAttribute<float>(InAttributeName, FGeometryCollection::VerticesGroup);
		for (int32 Idx = 0; Idx < NumPoints; ++Idx)
		{
			const int32 NewIdx = NumVertices + Idx;

			BoneMap[NewIdx] = 0;
			Vertex[NewIdx] = (FVector3f)InPoints[Idx];
			AttrArr[NewIdx] = InValues[Idx];
		}
	}

	void FPointsCollectionFacade::AddPointsWithIntAttribute(const TArray<FVector>& InPoints, const FName InAttributeName, const TArray<int32>& InValues)
	{
		const int32 NumPoints = InPoints.Num();
		const int32 NumVertices = Collection->NumElements(FGeometryCollection::VerticesGroup);

		AddPoints(InPoints);

		TManagedArray<int32>& BoneMap = Collection->AddAttribute<int32>("BoneMap", FGeometryCollection::VerticesGroup);
		TManagedArray<FVector3f>& Vertex = Collection->AddAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup);

		TManagedArray<int32>& AttrArr = Collection->AddAttribute<int32>(InAttributeName, FGeometryCollection::VerticesGroup);
		for (int32 Idx = 0; Idx < NumPoints; ++Idx)
		{
			const int32 NewIdx = NumVertices + Idx;

			BoneMap[NewIdx] = 0;
			Vertex[NewIdx] = (FVector3f)InPoints[Idx];
			AttrArr[NewIdx] = InValues[Idx];
		}
	}

	void FPointsCollectionFacade::AddPointsWithVectorAttribute(const TArray<FVector>& InPoints, const FName InAttributeName, const TArray<FVector>& InValues)
	{
		const int32 NumPoints = InPoints.Num();
		const int32 NumVertices = Collection->NumElements(FGeometryCollection::VerticesGroup);

		AddPoints(InPoints);

		TManagedArray<int32>& BoneMap = Collection->AddAttribute<int32>("BoneMap", FGeometryCollection::VerticesGroup);
		TManagedArray<FVector3f>& Vertex = Collection->AddAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup);

		TManagedArray<FVector>& AttrArr = Collection->AddAttribute<FVector>(InAttributeName, FGeometryCollection::VerticesGroup);
		for (int32 Idx = 0; Idx < NumPoints; ++Idx)
		{
			const int32 NewIdx = NumVertices + Idx;

			BoneMap[NewIdx] = 0;
			Vertex[NewIdx] = (FVector3f)InPoints[Idx];
			AttrArr[NewIdx] = InValues[Idx];
		}
	}

	void FPointsCollectionFacade::GetPointsWithFloatAttribute(TArray<FVector>& OutPoints, const FName InAttributeName, TArray<float>& OutValues, const int32 InTransformIndex)
	{
		TManagedArray<int32>& TransformToGeometryIndex = Collection->AddAttribute<int32>("TransformToGoemetryIndex", FGeometryCollection::TransformGroup);
		TManagedArray<int32>& VertexStartArr = Collection->AddAttribute<int32>("VertexStart", FGeometryCollection::GeometryGroup);
		TManagedArray<int32>& VertexCount = Collection->AddAttribute<int32>("VertexCount", FGeometryCollection::GeometryGroup);
		TManagedArray<FVector3f>& Vertex = Collection->AddAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup);
		TManagedArray<float>& AttrArr = Collection->AddAttribute<float>(InAttributeName, FGeometryCollection::VerticesGroup);

		const int32 GeometryIndex = TransformToGeometryIndex[InTransformIndex];
		const int32 VertexStart = VertexStartArr[GeometryIndex];
		const int32 NumPoints = VertexCount[GeometryIndex];

		OutPoints.Empty();
		OutPoints.AddUninitialized(NumPoints);

		OutValues.Empty();
		OutValues.AddUninitialized(NumPoints);

		for (int32 Idx = 0; Idx < NumPoints; ++Idx)
		{
			const int32 NewIdx = VertexStart + Idx;

			OutPoints[Idx] = (FVector)Vertex[NewIdx];
			OutValues[Idx] = AttrArr[NewIdx];
		}
	}
};


