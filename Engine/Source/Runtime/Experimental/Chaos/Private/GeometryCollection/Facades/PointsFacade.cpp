// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/Facades/PointsFacade.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/ManagedArrayCollection.h"

namespace GeometryCollection::Facades
{
	const FName FPointsFacade::PointsGroup = "Points";
	const FName FPointsFacade::PointAttribute("Point");

	FPointsFacade::FPointsFacade(FManagedArrayCollection& InCollection)
		: ConstCollection(InCollection)
		, MutableCollection(&InCollection)
	{
	}

	FPointsFacade::FPointsFacade(const FManagedArrayCollection& InCollection)
		: ConstCollection(InCollection)
		, MutableCollection(nullptr)
	{
	}

	/* ------------------------------------------------------------------------------------------------------------- */

	const FManagedArrayCollection& FPointsFacade::GetConstCollection() const
	{
		return ConstCollection;
	}
	/* ------------------------------------------------------------------------------------------------------------- */

	bool FPointsFacade::IsValid() const
	{
		if (ConstCollection.HasGroup(FPointsFacade::PointsGroup) &&
			ConstCollection.HasAttribute(FPointsFacade::PointAttribute, FPointsFacade::PointsGroup))
		{
			return true;
		}

		return false;
	}

	/* ------------------------------------------------------------------------------------------------------------- */

	void FPointsFacade::DefineSchema()
	{
		if (ensure(MutableCollection))
		{
			MutableCollection->AddGroup(FPointsFacade::PointsGroup);
			MutableCollection->AddAttribute<FVector3f>(FPointsFacade::PointAttribute, FPointsFacade::PointsGroup);
		}
	}

	/* ------------------------------------------------------------------------------------------------------------- */

	int32 FPointsFacade::GetNumPoints() const
	{
		if (IsValid())
		{
			return ConstCollection.NumElements(FPointsFacade::PointsGroup);
		}

		return 0;
	}

	/* ------------------------------------------------------------------------------------------------------------- */

	int32 FPointsFacade::GetNumAttrs() const
	{
		if (IsValid())
		{
			return ConstCollection.NumAttributes(FPointsFacade::PointsGroup);
		}

		return 0;
	}

	/* ------------------------------------------------------------------------------------------------------------- */

	TArray<FName> FPointsFacade::GetAttrs() const
	{
		TArray<FName> AttrArr;

		if (IsValid())
		{
			AttrArr = ConstCollection.AttributeNames(FPointsFacade::PointsGroup);
		}

		return AttrArr;
	}

	/* ------------------------------------------------------------------------------------------------------------- */

	FString FPointsFacade::GetAttrsAsString(bool bAddCommaSeparator) const
	{
		TStringBuilder<256> StringBuilder;


		if (IsValid())
		{
			TArray<FName> Attrs = ConstCollection.AttributeNames(FPointsFacade::PointsGroup);

			for (int32 Idx = 1; Idx < Attrs.Num(); ++Idx)
			{
				StringBuilder += Attrs[Idx].ToString();

				if (Idx < Attrs.Num() - 1)
				{
					if (bAddCommaSeparator)
					{
						StringBuilder += FString(", ");
					}
					else
					{
						StringBuilder += FString(" ");
					}
				}
			}
		}

		return StringBuilder.ToString();
	}

	/* ------------------------------------------------------------------------------------------------------------- */

	void FPointsFacade::AddPoints(const TArray<FVector3f>& InPoints)
	{
		if (ensure(MutableCollection))
		{
			const int32 NumPoints = InPoints.Num();

			DefineSchema();

			MutableCollection->AddElements(NumPoints, FPointsFacade::PointsGroup);

			TManagedArray<FVector3f>& Point = MutableCollection->ModifyAttribute<FVector3f>(FPointsFacade::PointAttribute, FPointsFacade::PointsGroup);

			for (int32 Idx = 0; Idx < NumPoints; ++Idx)
			{
				Point[Idx] = InPoints[Idx];
			}
		}
	}

	/* ------------------------------------------------------------------------------------------------------------- */

	void FPointsFacade::AddPoints(const TArray<FVector>& InPoints)
	{
		if (ensure(MutableCollection))
		{
			const int32 NumPoints = InPoints.Num();

			DefineSchema();

			MutableCollection->AddElements(NumPoints, FPointsFacade::PointsGroup);

			TManagedArray<FVector3f>& Point = MutableCollection->ModifyAttribute<FVector3f>(FPointsFacade::PointAttribute, FPointsFacade::PointsGroup);

			for (int32 Idx = 0; Idx < NumPoints; ++Idx)
			{
				Point[Idx] = FVector3f(InPoints[Idx]);
			}
		}
	}

	/* ------------------------------------------------------------------------------------------------------------- */

	void FPointsFacade::AddPoints(const FPointsFacade& InPointsFacade)
	{
		if (ensure(MutableCollection))
		{
			DefineSchema();

			InPointsFacade.ConstCollection.CopyTo(MutableCollection);
		}
	}

	/* ------------------------------------------------------------------------------------------------------------- */

	void FPointsFacade::AppendPoints(const TArray<FVector3f>& InPoints)
	{
		if (ensure(MutableCollection))
		{
			const int32 NumNewPoints = InPoints.Num();

			if (IsValid())
			{
				const int32 NumExistingPoints = ConstCollection.NumElements(FPointsFacade::PointsGroup);

				MutableCollection->AddElements(NumNewPoints, FPointsFacade::PointsGroup);

				TManagedArray<FVector3f>& Point = MutableCollection->ModifyAttribute<FVector3f>(FPointsFacade::PointAttribute, FPointsFacade::PointsGroup);

				for (int32 Idx = 0; Idx < NumNewPoints; ++Idx)
				{
					Point[NumExistingPoints + Idx] = InPoints[Idx];
				}
			}
		}
	}

	/* ------------------------------------------------------------------------------------------------------------- */

	void FPointsFacade::AppendPoints(const TArray<FVector>& InPoints)
	{
		if (ensure(MutableCollection))
		{
			const int32 NumNewPoints = InPoints.Num();

			if (IsValid())
			{
				const int32 NumExistingPoints = ConstCollection.NumElements(FPointsFacade::PointsGroup);

				MutableCollection->AddElements(NumNewPoints, FPointsFacade::PointsGroup);

				TManagedArray<FVector3f>& Point = MutableCollection->ModifyAttribute<FVector3f>(FPointsFacade::PointAttribute, FPointsFacade::PointsGroup);

				for (int32 Idx = 0; Idx < NumNewPoints; ++Idx)
				{
					Point[NumExistingPoints + Idx] = FVector3f(InPoints[Idx]);
				}
			}
		}
	}

	/* ------------------------------------------------------------------------------------------------------------- */

	void FPointsFacade::AppendPoints(const FPointsFacade& InPointsFacade)
	{
		if (ensure(MutableCollection))
		{
			if (IsValid())
			{
				if (InPointsFacade.ConstCollection.HasGroup(FPointsFacade::PointsGroup) &&
					InPointsFacade.ConstCollection.HasAttribute(FPointsFacade::PointAttribute, FPointsFacade::PointsGroup))
				{
					MutableCollection->Append(InPointsFacade.ConstCollection);
				}
			}
		}
	}

	/* ------------------------------------------------------------------------------------------------------------- */

	void FPointsFacade::DeletePoints(const TArray<int32>& InSortedDeletionList)
	{
		if (ensure(MutableCollection))
		{
			MutableCollection->RemoveElements(FPointsFacade::PointsGroup, InSortedDeletionList);
		}
	}

	/* ------------------------------------------------------------------------------------------------------------- */

	bool FPointsFacade::AddFloatAttribute(const FName InAttribute, const TArray<float>& InValues)
	{
		if (ensure(MutableCollection))
		{
			if (IsValid())
			{
				const TManagedArray<FVector3f>& Point = MutableCollection->GetAttribute<FVector3f>(FPointsFacade::PointAttribute, FPointsFacade::PointsGroup);
				const int32 NumPoints = FMath::Min(Point.Num(), InValues.Num());

				if (NumPoints == InValues.Num())
				{
					TManagedArray<float>* Attr = MutableCollection->FindAttributeTyped<float>(InAttribute, FPointsFacade::PointsGroup);
					if (!Attr)
					{
						TManagedArray<float>& AttrArr = MutableCollection->AddAttribute<float>(InAttribute, FPointsFacade::PointsGroup);

						for (int32 Idx = 0; Idx < NumPoints; ++Idx)
						{
							AttrArr[Idx] = InValues[Idx];
						}

						return true;
					}
				}
			}
		}

		return false;
	}

	/* ------------------------------------------------------------------------------------------------------------- */

	bool FPointsFacade::AddBoolAttribute(const FName InAttribute, const TArray<bool>& InValues)
	{
		if (ensure(MutableCollection))
		{
			if (IsValid())
			{
				const TManagedArray<FVector3f>& Point = MutableCollection->GetAttribute<FVector3f>(FPointsFacade::PointAttribute, FPointsFacade::PointsGroup);
				const int32 NumPoints = FMath::Min(Point.Num(), InValues.Num());

				if (NumPoints == InValues.Num())
				{
					TManagedArray<bool>* Attr = MutableCollection->FindAttributeTyped<bool>(InAttribute, FPointsFacade::PointsGroup);
					if (!Attr)
					{
						TManagedArray<bool>& AttrArr = MutableCollection->AddAttribute<bool>(InAttribute, FPointsFacade::PointsGroup);

						for (int32 Idx = 0; Idx < NumPoints; ++Idx)
						{
							AttrArr[Idx] = InValues[Idx];
						}

						return true;
					}
				}
			}
		}

		return false;
	}

	/* ------------------------------------------------------------------------------------------------------------- */

	bool FPointsFacade::AddBoolAttribute(const FName InAttribute, const TBitArray<>& InValues)
	{
		if (ensure(MutableCollection))
		{
			if (IsValid())
			{
				const TManagedArray<FVector3f>& Point = MutableCollection->GetAttribute<FVector3f>(FPointsFacade::PointAttribute, FPointsFacade::PointsGroup);
				const int32 NumPoints = FMath::Min(Point.Num(), InValues.Num());

				if (NumPoints == InValues.Num())
				{
					TManagedArray<bool>* Attr = MutableCollection->FindAttributeTyped<bool>(InAttribute, FPointsFacade::PointsGroup);
					if (!Attr)
					{
						TManagedArray<bool>& AttrArr = MutableCollection->AddAttribute<bool>(InAttribute, FPointsFacade::PointsGroup);

						for (int32 Idx = 0; Idx < NumPoints; ++Idx)
						{
							AttrArr[Idx] = InValues[Idx];
						}

						return true;
					}
				}
			}
		}

		return false;
	}

	/* ------------------------------------------------------------------------------------------------------------- */

	bool FPointsFacade::AddIntAttribute(const FName InAttribute, const TArray<int32>& InValues)
	{
		if (ensure(MutableCollection))
		{
			if (IsValid())
			{
				const TManagedArray<FVector3f>& Point = MutableCollection->GetAttribute<FVector3f>(FPointsFacade::PointAttribute, FPointsFacade::PointsGroup);
				const int32 NumPoints = FMath::Min(Point.Num(), InValues.Num());

				if (NumPoints == InValues.Num())
				{
					TManagedArray<int32>* Attr = MutableCollection->FindAttributeTyped<int32>(InAttribute, FPointsFacade::PointsGroup);
					if (!Attr)
					{
						TManagedArray<int32>& AttrArr = MutableCollection->AddAttribute<int32>(InAttribute, FPointsFacade::PointsGroup);

						for (int32 Idx = 0; Idx < NumPoints; ++Idx)
						{
							AttrArr[Idx] = InValues[Idx];
						}

						return true;
					}
				}
			}
		}

		return false;
	}

	/* ------------------------------------------------------------------------------------------------------------- */

	bool FPointsFacade::AddVector2Attribute(const FName InAttribute, const TArray<FVector2f>& InValues)
	{
		if (ensure(MutableCollection))
		{
			if (IsValid())
			{
				const TManagedArray<FVector3f>& Point = MutableCollection->GetAttribute<FVector3f>(FPointsFacade::PointAttribute, FPointsFacade::PointsGroup);
				const int32 NumPoints = FMath::Min(Point.Num(), InValues.Num());

				if (NumPoints == InValues.Num())
				{
					TManagedArray<FVector2f>* Attr = MutableCollection->FindAttributeTyped<FVector2f>(InAttribute, FPointsFacade::PointsGroup);
					if (!Attr)
					{
						TManagedArray<FVector2f>& AttrArr = MutableCollection->AddAttribute<FVector2f>(InAttribute, FPointsFacade::PointsGroup);

						for (int32 Idx = 0; Idx < NumPoints; ++Idx)
						{
							AttrArr[Idx] = InValues[Idx];
						}

						return true;
					}
				}
			}
		}

		return false;
	}

	/* ------------------------------------------------------------------------------------------------------------- */

	bool FPointsFacade::AddVector3Attribute(const FName InAttribute, const TArray<FVector3f>& InValues)
	{
		if (ensure(MutableCollection))
		{
			if (IsValid())
			{
				const TManagedArray<FVector3f>& Point = MutableCollection->GetAttribute<FVector3f>(FPointsFacade::PointAttribute, FPointsFacade::PointsGroup);
				const int32 NumPoints = FMath::Min(Point.Num(), InValues.Num());

				TManagedArray<FVector3f>* Attr = MutableCollection->FindAttributeTyped<FVector3f>(InAttribute, FPointsFacade::PointsGroup);
				if (!Attr)
				{
					TManagedArray<FVector3f>& AttrArr = MutableCollection->AddAttribute<FVector3f>(InAttribute, FPointsFacade::PointsGroup);

					for (int32 Idx = 0; Idx < NumPoints; ++Idx)
					{
						AttrArr[Idx] = InValues[Idx];
					}

					return true;
				}
			}
		}

		return false;
	}

	/* ------------------------------------------------------------------------------------------------------------- */

	bool FPointsFacade::HasAttribute(const FName& InAttribute) const
	{
		if (IsValid())
		{
			return ConstCollection.HasAttribute(InAttribute, FPointsFacade::PointsGroup);
		}

		return false;
	}

	/* ------------------------------------------------------------------------------------------------------------- */

	EManagedArrayType FPointsFacade::GetAttributeType(const FName& InAttribute) const
	{
		if (IsValid())
		{
			if (ConstCollection.HasAttribute(InAttribute, FPointsFacade::PointsGroup))
			{
				return ConstCollection.GetAttributeType(InAttribute, FPointsFacade::PointsGroup);
			}
		}

		return EManagedArrayType::FNoneType;
	}

	/* ------------------------------------------------------------------------------------------------------------- */

	void FPointsFacade::FilterByFloatAttributeToCollection(const FName InAttribute, const EFilterByAttributeOperation InOperation, float InValue, float InValue2, FManagedArrayCollection& OutCollection) const
	{
		if (IsValid())
		{
			const TManagedArray<FVector3f>& Point = ConstCollection.GetAttribute<FVector3f>(FPointsFacade::PointAttribute, FPointsFacade::PointsGroup);
			const int32 NumPoints = Point.Num();

			TArray<int32> OrigIdx;
			TArray<FVector>OutPoints;

			const TManagedArray<float>* AttrArr = ConstCollection.FindAttribute<float>(InAttribute, FPointsFacade::PointsGroup);
			if (AttrArr)
			{
				for (int32 Idx = 0; Idx < NumPoints; ++Idx)
				{
					if ((InOperation == EFilterByAttributeOperation::Equal && (*AttrArr)[Idx] == InValue) ||
						(InOperation == EFilterByAttributeOperation::NotEqual && (*AttrArr)[Idx] != InValue) ||
						(InOperation == EFilterByAttributeOperation::Greater && (*AttrArr)[Idx] > InValue) ||
						(InOperation == EFilterByAttributeOperation::GreaterOrEqual && (*AttrArr)[Idx] >= InValue) ||
						(InOperation == EFilterByAttributeOperation::Smaller && (*AttrArr)[Idx] < InValue) ||
						(InOperation == EFilterByAttributeOperation::SmallerOrEqual && (*AttrArr)[Idx] <= InValue) ||
						(InOperation == EFilterByAttributeOperation::InRangeInclusive && (*AttrArr)[Idx] >= InValue && (*AttrArr)[Idx] <= InValue2) ||
						(InOperation == EFilterByAttributeOperation::InRangeExclusive && (*AttrArr)[Idx] > InValue && (*AttrArr)[Idx] < InValue2))
					{
						OutPoints.Add(FVector(Point[Idx]));
						OrigIdx.Add(Idx);
					}
				}
			}

			const int32 NumPointsInOutputCollection = OutPoints.Num();

			if (OutCollection.HasGroup(FPointsFacade::PointsGroup))
			{
				OutCollection.RemoveGroup(FPointsFacade::PointsGroup);
			}

			OutCollection.AddGroup(FPointsFacade::PointsGroup);
			OutCollection.AddElements(NumPointsInOutputCollection, FPointsFacade::PointsGroup);

			if (!OutCollection.HasAttribute(FPointsFacade::PointAttribute, FPointsFacade::PointsGroup))
			{
				OutCollection.AddAttribute<FVector3f>(FPointsFacade::PointAttribute, FPointsFacade::PointsGroup);
			}

			// Set values for the Point attribute
			TManagedArray<FVector3f>& PointAttr = OutCollection.ModifyAttribute<FVector3f>(FPointsFacade::PointAttribute, FPointsFacade::PointsGroup);


			for (int32 Idx = 0; Idx < NumPointsInOutputCollection; ++Idx)
			{
				PointAttr[Idx] = FVector3f(OutPoints[Idx]);
			}

			// Add other attributes
			TArray<FName> Attrs = ConstCollection.AttributeNames(FPointsFacade::PointsGroup);

			for (int32 Idx = 1; Idx < Attrs.Num(); ++Idx)
			{
				const FName Attr = Attrs[Idx];

				if (ConstCollection.GetAttributeType(Attr, FPointsFacade::PointsGroup) == EManagedArrayType::FFloatType)
				{
					OutCollection.AddAttribute<float>(Attr, FPointsFacade::PointsGroup);

					const TManagedArray<float>& FloatAttr = ConstCollection.GetAttribute<float>(Attr, FPointsFacade::PointsGroup);
					TManagedArray<float>& NewAttr = OutCollection.ModifyAttribute<float>(Attr, FPointsFacade::PointsGroup);

					for (int32 Idx1 = 0; Idx1 < NumPointsInOutputCollection; ++Idx1)
					{
						NewAttr[Idx1] = FloatAttr[OrigIdx[Idx1]];
					}
				}
				else if (ConstCollection.GetAttributeType(Attrs[Idx], FPointsFacade::PointsGroup) == EManagedArrayType::FBoolType)
				{
					OutCollection.AddAttribute<bool>(Attr, FPointsFacade::PointsGroup);

					const TManagedArray<bool>& FloatAttr = ConstCollection.GetAttribute<bool>(Attr, FPointsFacade::PointsGroup);
					TManagedArray<bool>& NewAttr = OutCollection.ModifyAttribute<bool>(Attr, FPointsFacade::PointsGroup);

					for (int32 Idx1 = 0; Idx1 < NumPointsInOutputCollection; ++Idx1)
					{
						NewAttr[Idx1] = FloatAttr[OrigIdx[Idx1]];
					}
				}
				else if (ConstCollection.GetAttributeType(Attrs[Idx], FPointsFacade::PointsGroup) == EManagedArrayType::FInt32Type)
				{
					OutCollection.AddAttribute<int32>(Attr, FPointsFacade::PointsGroup);

					const TManagedArray<int32>& FloatAttr = ConstCollection.GetAttribute<int32>(Attr, FPointsFacade::PointsGroup);
					TManagedArray<int32>& NewAttr = OutCollection.ModifyAttribute<int32>(Attr, FPointsFacade::PointsGroup);

					for (int32 Idx1 = 0; Idx1 < NumPointsInOutputCollection; ++Idx1)
					{
						NewAttr[Idx1] = FloatAttr[OrigIdx[Idx1]];
					}
				}
				else if (ConstCollection.GetAttributeType(Attrs[Idx], FPointsFacade::PointsGroup) == EManagedArrayType::FVectorType)
				{
					OutCollection.AddAttribute<FVector3f>(Attr, FPointsFacade::PointsGroup);

					const TManagedArray<FVector3f>& FloatAttr = ConstCollection.GetAttribute<FVector3f>(Attr, FPointsFacade::PointsGroup);
					TManagedArray<FVector3f>& NewAttr = OutCollection.ModifyAttribute<FVector3f>(Attr, FPointsFacade::PointsGroup);

					for (int32 Idx1 = 0; Idx1 < NumPointsInOutputCollection; ++Idx1)
					{
						NewAttr[Idx1] = FloatAttr[OrigIdx[Idx1]];
					}
				}
				else if (ConstCollection.GetAttributeType(Attrs[Idx], FPointsFacade::PointsGroup) == EManagedArrayType::FVector2DType)
				{
					OutCollection.AddAttribute<FVector2f>(Attr, FPointsFacade::PointsGroup);

					const TManagedArray<FVector2f>& FloatAttr = ConstCollection.GetAttribute<FVector2f>(Attr, FPointsFacade::PointsGroup);
					TManagedArray<FVector2f>& NewAttr = OutCollection.ModifyAttribute<FVector2f>(Attr, FPointsFacade::PointsGroup);

					for (int32 Idx1 = 0; Idx1 < NumPointsInOutputCollection; ++Idx1)
					{
						NewAttr[Idx1] = FloatAttr[OrigIdx[Idx1]];
					}
				}
			}
		}
	}

	/* ------------------------------------------------------------------------------------------------------------- */

	void FPointsFacade::FilterByFloatAttributeToArray(const FName InAttribute, const EFilterByAttributeOperation InOperation, float InValue, float InValue2, TArray<FVector>& OutPoints) const
	{
		if (IsValid())
		{
			const TManagedArray<FVector3f>& Point = ConstCollection.GetAttribute<FVector3f>(FPointsFacade::PointAttribute, FPointsFacade::PointsGroup);
			const int32 NumPoints = Point.Num();

			const TManagedArray<float>* AttrArr = ConstCollection.FindAttribute<float>(InAttribute, FPointsFacade::PointsGroup);
			if (AttrArr)
			{
				for (int32 Idx = 0; Idx < NumPoints; ++Idx)
				{
					if ((InOperation == EFilterByAttributeOperation::Equal && (*AttrArr)[Idx] == InValue) ||
						(InOperation == EFilterByAttributeOperation::NotEqual && (*AttrArr)[Idx] != InValue) ||
						(InOperation == EFilterByAttributeOperation::Greater && (*AttrArr)[Idx] > InValue) ||
						(InOperation == EFilterByAttributeOperation::GreaterOrEqual && (*AttrArr)[Idx] >= InValue) ||
						(InOperation == EFilterByAttributeOperation::Smaller && (*AttrArr)[Idx] < InValue) ||
						(InOperation == EFilterByAttributeOperation::SmallerOrEqual && (*AttrArr)[Idx] <= InValue) ||
						(InOperation == EFilterByAttributeOperation::InRangeInclusive && (*AttrArr)[Idx] >= InValue && (*AttrArr)[Idx] <= InValue2) ||
						(InOperation == EFilterByAttributeOperation::InRangeExclusive && (*AttrArr)[Idx] > InValue && (*AttrArr)[Idx] < InValue2))
					{
						OutPoints.Add(FVector(Point[Idx]));
					}
				}
			}
		}
	}

	/* ------------------------------------------------------------------------------------------------------------- */

	void FPointsFacade::FilterByFloatAttributeToSelection(const FName InAttribute, const EFilterByAttributeOperation InOperation, float InValue, float InValue2, TArray<int32>& OutPointSelection) const
	{
		if (IsValid())
		{
			const TManagedArray<FVector3f>& Point = ConstCollection.GetAttribute<FVector3f>(FPointsFacade::PointAttribute, FPointsFacade::PointsGroup);
			const int32 NumPoints = Point.Num();

			const TManagedArray<float>* AttrArr = ConstCollection.FindAttribute<float>(InAttribute, FPointsFacade::PointsGroup);
			if (AttrArr)
			{
				for (int32 Idx = 0; Idx < NumPoints; ++Idx)
				{
					if ((InOperation == EFilterByAttributeOperation::Equal && (*AttrArr)[Idx] == InValue) ||
						(InOperation == EFilterByAttributeOperation::NotEqual && (*AttrArr)[Idx] != InValue) ||
						(InOperation == EFilterByAttributeOperation::Greater && (*AttrArr)[Idx] > InValue) ||
						(InOperation == EFilterByAttributeOperation::GreaterOrEqual && (*AttrArr)[Idx] >= InValue) ||
						(InOperation == EFilterByAttributeOperation::Smaller && (*AttrArr)[Idx] < InValue) ||
						(InOperation == EFilterByAttributeOperation::SmallerOrEqual && (*AttrArr)[Idx] <= InValue) ||
						(InOperation == EFilterByAttributeOperation::InRangeInclusive && (*AttrArr)[Idx] >= InValue && (*AttrArr)[Idx] <= InValue2) ||
						(InOperation == EFilterByAttributeOperation::InRangeExclusive && (*AttrArr)[Idx] > InValue && (*AttrArr)[Idx] < InValue2))
					{
						OutPointSelection.Add(Idx);
					}
				}
			}
		}
	}

	/* ------------------------------------------------------------------------------------------------------------- */

	TArray<FVector> FPointsFacade::GetPointsAsArray() const
	{
		TArray<FVector> OutArray;

		if (IsValid())
		{
			const TManagedArray<FVector3f>& Point = ConstCollection.GetAttribute<FVector3f>(FPointsFacade::PointAttribute, FPointsFacade::PointsGroup);
			const int32 NumPoints = Point.Num();

			OutArray.SetNumUninitialized(NumPoints);

			for (int32 Idx = 0; Idx < NumPoints; ++Idx)
			{
				OutArray[Idx] = FVector(Point[Idx]);
			}
		}

		return OutArray;
	}

	/* ------------------------------------------------------------------------------------------------------------- */

	TArray<FVector3f> FPointsFacade::GetPointsAsFloatArray() const
	{
		return TArray<FVector3f>(GetPointsAsArray());
	}

	/* ------------------------------------------------------------------------------------------------------------- */

	const TArray<float>& FPointsFacade::GetFloatAttributeValues(const FName InAttribute) const
	{
		static TArray<float> EmptyArray;

		if (const TManagedArray<float>* Attr = ConstCollection.FindAttribute<float>(InAttribute, FPointsFacade::PointsGroup))
		{
			return Attr->GetConstArray();
		}

		return EmptyArray;
	}

	/* ------------------------------------------------------------------------------------------------------------- */

	const TBitArray<>& FPointsFacade::GetBoolAttributeValues(const FName InAttribute) const
	{
		static TBitArray<> EmptyArray;

		if (const TManagedArray<bool>* Attr = ConstCollection.FindAttribute<bool>(InAttribute, FPointsFacade::PointsGroup))
		{
			return Attr->GetConstArray();
		}

		return EmptyArray;
	}

	/* ------------------------------------------------------------------------------------------------------------- */

	const TArray<int32>& FPointsFacade::GetIntAttributeValues(const FName InAttribute) const
	{
		static TArray<int32> EmptyArray;

		if (const TManagedArray<int32>* Attr = ConstCollection.FindAttribute<int32>(InAttribute, FPointsFacade::PointsGroup))
		{
			return Attr->GetConstArray();
		}

		return EmptyArray;
	}

	/* ------------------------------------------------------------------------------------------------------------- */

	const TArray<FVector2f>& FPointsFacade::GetVector2AttributeValues(const FName InAttribute) const
	{
		static TArray<FVector2f> EmptyArray;

		if (const TManagedArray<FVector2f>* Attr = ConstCollection.FindAttribute<FVector2f>(InAttribute, FPointsFacade::PointsGroup))
		{
			return Attr->GetConstArray();
		}

		return EmptyArray;
	}

	/* ------------------------------------------------------------------------------------------------------------- */

	const TArray<FVector3f>& FPointsFacade::GetVector3AttributeValues(const FName InAttribute) const
	{
		static TArray<FVector3f> EmptyArray;

		if (const TManagedArray<FVector3f>* Attr = ConstCollection.FindAttribute<FVector3f>(InAttribute, FPointsFacade::PointsGroup))
		{
			return Attr->GetConstArray();
		}

		return EmptyArray;
	}

	/* ------------------------------------------------------------------------------------------------------------- */

	const float FPointsFacade::GetFloatAttributeValue(const FName InAttribute, const int32 ElementIndex) const
	{
		float Element = 0.0;

		if (const TManagedArray<float>* Attr = ConstCollection.FindAttribute<float>(InAttribute, FPointsFacade::PointsGroup))
		{
			if (Attr->IsValidIndex(ElementIndex))
			{
				Element = Attr->GetConstArray()[ElementIndex];
			}
		}

		return Element;
	}

	/* ------------------------------------------------------------------------------------------------------------- */

	const bool FPointsFacade::GetBoolAttributeValue(const FName InAttribute, const int32 ElementIndex) const
	{
		bool Element = false;

		if (const TManagedArray<bool>* Attr = ConstCollection.FindAttribute<bool>(InAttribute, FPointsFacade::PointsGroup))
		{
			if (Attr->IsValidIndex(ElementIndex))
			{
				Element = Attr->GetConstArray()[ElementIndex];
			}
		}

		return Element;
	}

	/* ------------------------------------------------------------------------------------------------------------- */

	const int32 FPointsFacade::GetIntAttributeValue(const FName InAttribute, const int32 ElementIndex) const
	{
		int32 Element = 0;

		if (const TManagedArray<int32>* Attr = ConstCollection.FindAttribute<int32>(InAttribute, FPointsFacade::PointsGroup))
		{
			if (Attr->IsValidIndex(ElementIndex))
			{
				Element = Attr->GetConstArray()[ElementIndex];
			}
		}

		return Element;
	}

	/* ------------------------------------------------------------------------------------------------------------- */

	const FVector2f FPointsFacade::GetVector2AttributeValue(const FName InAttribute, const int32 ElementIndex) const
	{
		FVector2f Element = FVector2f::ZeroVector;

		if (const TManagedArray<FVector2f>* Attr = ConstCollection.FindAttribute<FVector2f>(InAttribute, FPointsFacade::PointsGroup))
		{
			if (Attr->IsValidIndex(ElementIndex))
			{
				Element = Attr->GetConstArray()[ElementIndex];
			}
		}

		return Element;
	}

	/* ------------------------------------------------------------------------------------------------------------- */

	const FVector3f FPointsFacade::GetVector3AttributeValue(const FName InAttribute, const int32 ElementIndex) const
	{
		FVector3f Element = FVector3f::ZeroVector;

		if (const TManagedArray<FVector3f>* Attr = ConstCollection.FindAttribute<FVector3f>(InAttribute, FPointsFacade::PointsGroup))
		{
			if (Attr->IsValidIndex(ElementIndex))
			{
				Element = Attr->GetConstArray()[ElementIndex];
			}
		}

		return Element;
	}
	
	/* ------------------------------------------------------------------------------------------------------------- */

	void FPointsFacade::TransformPoints(const FTransform& InTransform)
	{
		if (ensure(MutableCollection))
		{
			if (IsValid())
			{
				TManagedArray<FVector3f>& PointsArr = MutableCollection->ModifyAttribute<FVector3f>(FPointsFacade::PointAttribute, FPointsFacade::PointsGroup);

				const int32 NumPoints = GetNumPoints();

				if (NumPoints > 0)
				{
					for (int32 Idx = 0; Idx < NumPoints; ++Idx)
					{
						PointsArr[Idx] = FVector3f(InTransform.TransformPosition(FVector(PointsArr[Idx])));
					}
				}
			}
		}
	}

	/* ------------------------------------------------------------------------------------------------------------- */

	bool FPointsFacade::GeneratePointsInBox(const FBox& InBox, float InPointSeparation)
	{
		if (InPointSeparation < 1.f)
		{
			InPointSeparation = 1.f;
		}

		FVector Extent = 2.f * InBox.GetExtent();
		FVector Center = InBox.GetCenter();

		FVector NewExtent;
		NewExtent.X = (FMath::CeilToDouble(Extent.X / InPointSeparation) + 1.f) * InPointSeparation;
		NewExtent.Y = (FMath::CeilToDouble(Extent.Y / InPointSeparation) + 1.f) * InPointSeparation;
		NewExtent.Z = (FMath::CeilToDouble(Extent.Z / InPointSeparation) + 1.f) * InPointSeparation;

		FVector Min = Center - 0.5 * NewExtent;
		FVector Max = Center + 0.5 * NewExtent;

		const int32 NumX = FMath::Min(1000, int32(NewExtent.X / InPointSeparation)) + 1;
		const int32 NumY = FMath::Min(1000, int32(NewExtent.Y / InPointSeparation)) + 1;
		const int32 NumZ = FMath::Min(1000, int32(NewExtent.Z / InPointSeparation)) + 1;

		const uint32 NumPoints = NumX * NumY * NumZ;
		if (NumPoints > 0)
		{
			TArray<FVector> Points;
			Points.Reserve(NumPoints);

			for (int32 IdxZ = 0; IdxZ < NumZ; ++IdxZ)
			{
				for (int32 IdxX = 0; IdxX < NumX; ++IdxX)
				{
					for (int32 IdxY = 0; IdxY < NumY; ++IdxY)
					{
						const double X = Min.X + double(IdxX) * InPointSeparation;
						const double Y = Min.Y + double(IdxY) * InPointSeparation;
						const double Z = Min.Z + double(IdxZ) * InPointSeparation;

						Points.Add(FVector(X, Y, Z));
					}
				}
			}

			AddPoints(Points);

			return true;
		}

		return false;
	}

	/* ------------------------------------------------------------------------------------------------------------- */

	bool FPointsFacade::GeneratePointsOnPlane(const FBox& InBox, const int32 InPlaneOrientation, float InPointSeparation, float InOffset)
	{
		if (InPointSeparation < 1.0)
		{
			InPointSeparation = 1.f;
		}

		InOffset = FMath::Clamp(InOffset, -1.f, 1.f);

		FVector Extent = 2.f * InBox.GetExtent();
		FVector Center = InBox.GetCenter();

		FVector NewExtent;
		NewExtent.X = (FMath::CeilToDouble(Extent.X / InPointSeparation) + 1.f) * InPointSeparation;
		NewExtent.Y = (FMath::CeilToDouble(Extent.Y / InPointSeparation) + 1.f) * InPointSeparation;
		NewExtent.Z = (FMath::CeilToDouble(Extent.Z / InPointSeparation) + 1.f) * InPointSeparation;

		FVector Min = Center - 0.5 * NewExtent;
		FVector Max = Center + 0.5 * NewExtent;

		const int32 NumX = FMath::Min(1000, int32(NewExtent.X / InPointSeparation)) + 1;
		const int32 NumY = FMath::Min(1000, int32(NewExtent.Y / InPointSeparation)) + 1;
		const int32 NumZ = FMath::Min(1000, int32(NewExtent.Z / InPointSeparation)) + 1;

		TArray<FVector> Points;
		int32 NumPoints = 0;

		// XYPlane
		if (InPlaneOrientation == 0)
		{
			NumPoints = NumX * NumY;
			if (NumPoints > 0)
			{
				Points.Reserve(NumPoints);

				for (int32 IdxX = 0; IdxX < NumX; ++IdxX)
				{
					for (int32 IdxY = 0; IdxY < NumY; ++IdxY)
					{
						const double X = Min.X + double(IdxX) * InPointSeparation;
						const double Y = Min.Y + double(IdxY) * InPointSeparation;
						const double Z = Center.Z + InOffset * 0.5 * Extent.Z;

						Points.Add(FVector(X, Y, Z));
					}
				}
			}
		}
		// YZPlane
		else if (InPlaneOrientation == 1)
		{
			NumPoints = NumY * NumZ;
			if (NumPoints > 0)
			{
				Points.Reserve(NumPoints);

				for (int32 IdxY = 0; IdxY < NumY; ++IdxY)
				{
					for (int32 IdxZ = 0; IdxZ < NumZ; ++IdxZ)
					{
						const double X = Center.X + InOffset * 0.5 * Extent.X;
						const double Y = Min.Y + double(IdxY) * InPointSeparation;
						const double Z = Min.Z + double(IdxZ) * InPointSeparation;

						Points.Add(FVector(X, Y, Z));
					}
				}
			}
		}
		// ZXPlane
		else if (InPlaneOrientation == 2)
		{
			NumPoints = NumX * NumZ;
			if (NumPoints > 0)
			{
				Points.Reserve(NumPoints);

				for (int32 IdxZ = 0; IdxZ < NumZ; ++IdxZ)
				{
					for (int32 IdxX = 0; IdxX < NumX; ++IdxX)
					{
						const double X = Min.X + double(IdxX) * InPointSeparation;
						const double Y = Center.Y + InOffset * 0.5 * Extent.Y;
						const double Z = Min.Z + double(IdxZ) * InPointSeparation;

						Points.Add(FVector(X, Y, Z));
					}
				}
			}
		}

		AddPoints(Points);

		return NumPoints > 0;
	}

	/* ------------------------------------------------------------------------------------------------------------- */

	FBox FPointsFacade::GetBoundingBox()
	{
		if (GetNumPoints() > 0)
		{
			FBox BoundingBox = FBox(ForceInit);

			TArray<FVector> Points = GetPointsAsArray();

			for (FVector& Point : Points)
			{
				BoundingBox += Point;
			}

			return BoundingBox;
		}

		return FBox(ForceInit);
	}

	/* ------------------------------------------------------------------------------------------------------------- */

	FSphere FPointsFacade::GetBoundingSphere()
	{
		if (GetNumPoints() > 0)
		{
			TArray<FVector> Points = GetPointsAsArray();

			FSphere BoundingSphere(&Points[0], Points.Num());

			return BoundingSphere;
		}

		return FSphere(ForceInit);
	}

	/* ------------------------------------------------------------------------------------------------------------- */

}