// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GeometryCollection.cpp: FGeometryCollection methods.
=============================================================================*/

#include "GeometryCollection/GeometryCollection.h"

#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/GeometryCollectionUtility.h"
#include "GeometryCollection/GeometryCollectionProximityUtility.h"
#include "GeometryCollection/GeometryCollectionClusteringUtility.h"
#include "GeometryCollection/GeometryCollectionConvexUtility.h"
#include "GeometryCollection/Facades/CollectionHierarchyFacade.h"
#include "GeometryCollection/Facades/CollectionRenderingFacade.h"
#include "UObject/FortniteSeasonBranchObjectVersion.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include <iostream>
#include <fstream>
#include "Chaos/ChaosArchive.h"
#include "Voronoi/Voronoi.h"

bool bChaosGeometryCollectionEnableCollisionParticles = true;
FAutoConsoleVariableRef CVarChaosGeometryCollectionEnableCollisionParticles(
	TEXT("p.Chaos.GC.EnableCollisionParticles"),
	bChaosGeometryCollectionEnableCollisionParticles,
	TEXT("Enable use of collision particles for collision [def:true]"));

DEFINE_LOG_CATEGORY_STATIC(FGeometryCollectionLogging, Log, All);

// @todo: update names 
const FName FGeometryCollection::FacesGroup = "Faces";
const FName FGeometryCollection::GeometryGroup = "Geometry";
const FName FGeometryCollection::VerticesGroup = "Vertices";
const FName FGeometryCollection::BreakingGroup = "Breaking";
const FName FGeometryCollection::MaterialGroup = "Material";

// Attributes
const FName FGeometryCollection::SimulatableParticlesAttribute("SimulatableParticlesAttribute");
const FName FGeometryCollection::SimulationTypeAttribute("SimulationType");
const FName FGeometryCollection::StatusFlagsAttribute("StatusFlags");
const FName FGeometryCollection::ExternalCollisionsAttribute("ExternalCollisions");
const FName FGeometryCollection::ColorAttribute("Color");
const FName FGeometryCollection::BoneColorAttribute("BoneColor");
const FName FGeometryCollection::TransformToGeometryIndexAttribute("TransformToGeometryIndex");
const FName FGeometryCollection::BoundingBoxAttribute("BoundingBox");
const FName FGeometryCollection::TransformIndexAttribute("TransformIndex");
const FName FGeometryCollection::VertexStartAttribute("VertexStart");
const FName FGeometryCollection::VertexCountAttribute("VertexCount");
const FName FGeometryCollection::InnerRadiusAttribute("InnerRadius");
const FName FGeometryCollection::OuterRadiusAttribute("OuterRadius");
const FName FGeometryCollection::FaceStartAttribute("FaceStart");
const FName FGeometryCollection::FaceCountAttribute("FaceCount");
const FName FGeometryCollection::FaceIndicesAttribute("Indices");
const FName FGeometryCollection::FaceVisibleAttribute("Visible");
const FName FGeometryCollection::MaterialIndexAttribute("MaterialIndex");
const FName FGeometryCollection::MaterialIDAttribute("MaterialID");
const FName FGeometryCollection::VertexPositionAttribute("Vertex");
const FName FGeometryCollection::VertexNormalAttribute("Normal");
const FName FGeometryCollection::VertexTangentUAttribute("TangentU");
const FName FGeometryCollection::VertexTangentVAttribute("TangentV");
const FName FGeometryCollection::VertexBoneMapAttribute("BoneMap");

bool FGeometryCollection::AreCollisionParticlesEnabled()
{
	return bChaosGeometryCollectionEnableCollisionParticles;
}


FGeometryCollection::FGeometryCollection(FGeometryCollectionDefaults InDefaults)
	: FTransformCollection()
	, FGeometryCollectionConvexPropertiesInterface(this)
	, FGeometryCollectionProximityPropertiesInterface(this)
	, Defaults(InDefaults)
{
	Construct();
}

FGeometryCollection::~FGeometryCollection() = default;

void FGeometryCollection::DefineGeometrySchema(FManagedArrayCollection& InCollection)
{
	FTransformCollection::DefineTransformSchema(InCollection);

	FManagedArrayCollection::FConstructionParameters TransformDependency(FTransformCollection::TransformGroup);
	FManagedArrayCollection::FConstructionParameters VerticesDependency(FGeometryCollection::VerticesGroup);
	FManagedArrayCollection::FConstructionParameters FacesDependency(FGeometryCollection::FacesGroup);

	// Transform Group
	InCollection.AddAttribute<int32>("TransformToGeometryIndex", FTransformCollection::TransformGroup);
	InCollection.AddAttribute<int32>("SimulationType", FTransformCollection::TransformGroup);
	InCollection.AddAttribute<int32>("StatusFlags", FTransformCollection::TransformGroup);
	InCollection.AddAttribute<int32>("InitialDynamicState", FTransformCollection::TransformGroup);
	InCollection.AddAttribute<int32>("ExemplarIndex", FTransformCollection::TransformGroup);

	// Vertices Group
	InCollection.AddAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup);
	InCollection.AddAttribute<FVector3f>("Normal", FGeometryCollection::VerticesGroup);
	InCollection.AddAttribute<FLinearColor>(FGeometryCollection::ColorAttribute, FGeometryCollection::VerticesGroup);
	InCollection.AddAttribute<FVector3f>("TangentU", FGeometryCollection::VerticesGroup);
	InCollection.AddAttribute<FVector3f>("TangentV", FGeometryCollection::VerticesGroup);
	InCollection.AddAttribute<int32>("BoneMap", FGeometryCollection::VerticesGroup, TransformDependency);
	GeometryCollection::UV::DefineUVSchema(InCollection);

	// Faces Group
	InCollection.AddAttribute<FIntVector>("Indices", FGeometryCollection::FacesGroup, VerticesDependency);
	InCollection.AddAttribute<bool>("Visible", FGeometryCollection::FacesGroup);
	InCollection.AddAttribute<int32>("MaterialIndex", FGeometryCollection::FacesGroup);
	InCollection.AddAttribute<int32>("MaterialID", FGeometryCollection::FacesGroup);
	InCollection.AddAttribute<bool>("Internal", FGeometryCollection::FacesGroup);

	// Geometry Group
	InCollection.AddAttribute<int32>("TransformIndex", FGeometryCollection::GeometryGroup, TransformDependency);
	InCollection.AddAttribute<FBox>("BoundingBox", FGeometryCollection::GeometryGroup);
	InCollection.AddAttribute<float>("InnerRadius", FGeometryCollection::GeometryGroup);
	InCollection.AddAttribute<float>("OuterRadius", FGeometryCollection::GeometryGroup);
	InCollection.AddAttribute<int32>("VertexStart", FGeometryCollection::GeometryGroup, VerticesDependency);
	InCollection.AddAttribute<int32>("VertexCount", FGeometryCollection::GeometryGroup);
	InCollection.AddAttribute<int32>("FaceStart", FGeometryCollection::GeometryGroup, FacesDependency);
	InCollection.AddAttribute<int32>("FaceCount", FGeometryCollection::GeometryGroup);

	// Material Group
	InCollection.AddAttribute<FGeometryCollectionSection>("Sections", FGeometryCollection::MaterialGroup, FacesDependency);
}

void FGeometryCollection::Construct()
{
	Version = GetLatestVersionNumber();

	FManagedArrayCollection::FConstructionParameters TransformDependency(FTransformCollection::TransformGroup);
	FManagedArrayCollection::FConstructionParameters VerticesDependency(FGeometryCollection::VerticesGroup);
	FManagedArrayCollection::FConstructionParameters FacesDependency(FGeometryCollection::FacesGroup);

	// Transform Group
	AddExternalAttribute<int32>("TransformToGeometryIndex", FTransformCollection::TransformGroup, TransformToGeometryIndex);
	AddExternalAttribute<int32>("SimulationType", FTransformCollection::TransformGroup, SimulationType);
	AddExternalAttribute<int32>("StatusFlags", FTransformCollection::TransformGroup, StatusFlags);
	AddExternalAttribute<int32>("InitialDynamicState", FTransformCollection::TransformGroup, InitialDynamicState);
	AddExternalAttribute<int32>("ExemplarIndex", FTransformCollection::TransformGroup, ExemplarIndex);

	// Vertices Group
	AddExternalAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup, Vertex);
	AddExternalAttribute<FVector3f>("Normal", FGeometryCollection::VerticesGroup, Normal);
	AddExternalAttribute<FLinearColor>(FGeometryCollection::ColorAttribute, FGeometryCollection::VerticesGroup, Color);
	AddExternalAttribute<FVector3f>("TangentU", FGeometryCollection::VerticesGroup, TangentU);
	AddExternalAttribute<FVector3f>("TangentV", FGeometryCollection::VerticesGroup, TangentV);
	AddExternalAttribute<int32>("BoneMap", FGeometryCollection::VerticesGroup, BoneMap, TransformDependency);
	GeometryCollection::UV::DefineUVSchema(*this);

	// Faces Group
	AddExternalAttribute<FIntVector>("Indices", FGeometryCollection::FacesGroup, Indices, VerticesDependency);
	AddExternalAttribute<bool>("Visible", FGeometryCollection::FacesGroup, Visible);
	AddExternalAttribute<int32>("MaterialIndex", FGeometryCollection::FacesGroup, MaterialIndex);
	AddExternalAttribute<int32>("MaterialID", FGeometryCollection::FacesGroup, MaterialID);
	AddExternalAttribute<bool>("Internal", FGeometryCollection::FacesGroup, Internal);

	// Geometry Group
	AddExternalAttribute<int32>("TransformIndex", FGeometryCollection::GeometryGroup, TransformIndex, TransformDependency);
	AddExternalAttribute<FBox>("BoundingBox", FGeometryCollection::GeometryGroup, BoundingBox);
	AddExternalAttribute<float>("InnerRadius", FGeometryCollection::GeometryGroup, InnerRadius);
	AddExternalAttribute<float>("OuterRadius", FGeometryCollection::GeometryGroup, OuterRadius);
	AddExternalAttribute<int32>("VertexStart", FGeometryCollection::GeometryGroup, VertexStart, VerticesDependency);
	AddExternalAttribute<int32>("VertexCount", FGeometryCollection::GeometryGroup, VertexCount);
	AddExternalAttribute<int32>("FaceStart", FGeometryCollection::GeometryGroup, FaceStart, FacesDependency);
	AddExternalAttribute<int32>("FaceCount", FGeometryCollection::GeometryGroup, FaceCount);

	// Material Group
	AddExternalAttribute<FGeometryCollectionSection>("Sections", FGeometryCollection::MaterialGroup, Sections, FacesDependency);

	InitializeInterfaces();
}

void FGeometryCollection::SetDefaults(FName Group, uint32 StartSize, uint32 NumElements)
{
	Super::SetDefaults(Group, StartSize, NumElements);

	if (Group == FTransformCollection::TransformGroup)
	{
		for (uint32 Idx = StartSize; Idx < StartSize + NumElements; ++Idx)
		{
			TransformToGeometryIndex[Idx] = FGeometryCollection::Invalid;
			Parent[Idx] = FGeometryCollection::Invalid;
			SimulationType[Idx] = FGeometryCollection::ESimulationTypes::FST_None;
			StatusFlags[Idx] = 0;
			InitialDynamicState[Idx] = static_cast<int32>(Chaos::EObjectStateType::Uninitialized);
			ExemplarIndex[Idx] = INDEX_NONE;
		}

		FGeometryCollectionConvexUtility::SetDefaults(this, Group, StartSize, NumElements);
	}
	else if (Group == FGeometryCollection::VerticesGroup)
	{
		for (uint32 Idx = StartSize; Idx < StartSize + NumElements; ++Idx)
		{
			Color[Idx] = Defaults.DefaultVertexColor;
		}
	}
}

void FGeometryCollection::Append(const FManagedArrayCollection& InCollection)
{	
	/* InCollection data is appended to the front*/
	Super::Append(InCollection);
	if (const FGeometryCollection* InTypedCollection = InCollection.Cast<FGeometryCollection>())
	{
		const int32 Offset = InTypedCollection->NumElements(GeometryGroup);
		const int32 OtherSize = InTypedCollection->NumElements(TransformGroup);
		const int32 Size = NumElements(TransformGroup);
		/*TransformToGeometryIndex does not have GeometryGroup dependency, update manually*/
		for (int32 Idx = OtherSize; Idx < Size; ++Idx)
		{
			if (TransformToGeometryIndex[Idx] != INDEX_NONE)
			{
				TransformToGeometryIndex[Idx] += Offset;
			}
		}
	}
}

void FGeometryCollection::AppendCollection(const FGeometryCollection& InCollection)
{
	Append(InCollection);
}

// MaterialIDOffset is based on the number of materials added by this append geometry call
int32 FGeometryCollection::AppendGeometry(const FGeometryCollection & Element, int32 MaterialIDOffset, bool ReindexAllMaterials, const FTransform& TransformRoot)
{
	// until we support a transform hierarchy this is just one.
	if (Element.NumElements(FGeometryCollection::TransformGroup) == 0)
	{
		return INDEX_NONE;
	}

	int NumTransforms = NumElements(FTransformCollection::TransformGroup);
	int NumNewTransforms = Element.NumElements(FTransformCollection::TransformGroup);

	int32 StartTransformIndex = Super::AppendTransform(Element, TransformRoot);
	check(NumTransforms == StartTransformIndex);

	check(Element.NumElements(FGeometryCollection::FacesGroup) > 0);
	check(Element.NumElements(FGeometryCollection::VerticesGroup) > 0);

	int NumNewVertices = Element.NumElements(FGeometryCollection::VerticesGroup);
	const TManagedArray<FVector3f>& ElementVertices = Element.Vertex;
	const TManagedArray<FVector3f>& ElementNormals = Element.Normal;
	const TManagedArray<FLinearColor>& ElementColors = Element.Color;
	const TManagedArray<FVector3f>& ElementTangentUs = Element.TangentU;
	const TManagedArray<FVector3f>& ElementTangentVs = Element.TangentV;
	const TManagedArray<int32>& ElementBoneMap = Element.BoneMap;

	const TManagedArray<FIntVector>& ElementIndices = Element.Indices;
	const TManagedArray<bool>& ElementVisible = Element.Visible;
	const TManagedArray<int32>& ElementMaterialIndex = Element.MaterialIndex;
	const TManagedArray<int32>& ElementMaterialID = Element.MaterialID;
	const TManagedArray<bool>& ElementInternal = Element.Internal;

	const TManagedArray<int32>& ElementTransformIndex = Element.TransformIndex;
	const TManagedArray<FBox>& ElementBoundingBox = Element.BoundingBox;
	const TManagedArray<float>& ElementInnerRadius = Element.InnerRadius;
	const TManagedArray<float>& ElementOuterRadius = Element.OuterRadius;
	const TManagedArray<int32>& ElementVertexStart = Element.VertexStart;
	const TManagedArray<int32>& ElementVertexCount = Element.VertexCount;
	const TManagedArray<int32>& ElementFaceStart = Element.FaceStart;
	const TManagedArray<int32>& ElementFaceCount = Element.FaceCount;

	const TManagedArray<FString>& ElementBoneName = Element.BoneName;
	const TManagedArray<FGeometryCollectionSection>& ElementSections = Element.Sections;

	const TManagedArray<int32>& ElementSimulationType = Element.SimulationType;
	const TManagedArray<int32>& ElementStatusFlags = Element.StatusFlags;
	const TManagedArray<int32>& ElementInitialDynamicState = Element.InitialDynamicState;
	const TManagedArray<int32>& ElementExemplarIndex = Element.ExemplarIndex;

	// --- TRANSFORM ---
	for (int TransformIdx = 0; TransformIdx < NumNewTransforms; TransformIdx++)
	{
		SimulationType[TransformIdx + StartTransformIndex] = ElementSimulationType[TransformIdx];
		StatusFlags[TransformIdx + StartTransformIndex] = ElementStatusFlags[TransformIdx];
		InitialDynamicState[TransformIdx + StartTransformIndex] = ElementInitialDynamicState[TransformIdx];
		ExemplarIndex[TransformIdx + StartTransformIndex] = ElementExemplarIndex[TransformIdx];
	}

	// --- VERTICES GROUP ---

	int NumVertices = NumElements(FGeometryCollection::VerticesGroup);
	int VerticesIndex = AddElements(NumNewVertices, FGeometryCollection::VerticesGroup);
	TManagedArray<FVector3f>& Vertices = Vertex;
	TManagedArray<FVector3f>& Normals = Normal;
	TManagedArray<FLinearColor>& Colors = Color;
	TManagedArray<FVector3f>& TangentUs = TangentU;
	TManagedArray<FVector3f>& TangentVs = TangentV;
	TManagedArray<int32>& BoneMaps = BoneMap;
	TManagedArray<FIntVector>& FaceIndices = Indices;

	// Make sure we have enough UV layers to copy all the Element's UVs
	int32 ExistingUVLayers = NumUVLayers();
	int32 ElementUVLayers = Element.NumUVLayers();
	if (ElementUVLayers > ExistingUVLayers)
	{
		SetNumUVLayers(ElementUVLayers);
	}

	for (int vdx = 0; vdx < NumNewVertices; vdx++)
	{
		Vertices[VerticesIndex + vdx] = ElementVertices[vdx];
		Normals[VerticesIndex + vdx] = ElementNormals[vdx];
		
		for (int UVLayerIndex = 0; UVLayerIndex < ElementUVLayers; UVLayerIndex++)
		{
			ModifyUV(VerticesIndex + vdx, UVLayerIndex) = Element.GetUV(vdx, UVLayerIndex);
		}

		Colors[VerticesIndex + vdx] = ElementColors[vdx];
		TangentUs[VerticesIndex + vdx] = ElementTangentUs[vdx];
		TangentVs[VerticesIndex + vdx] = ElementTangentVs[vdx];
		BoneMaps[VerticesIndex + vdx] = ElementBoneMap[vdx] + StartTransformIndex;
	}

	// --- FACES GROUP ---


	int NumIndices = NumElements(FGeometryCollection::FacesGroup);
	int NumNewIndices = ElementIndices.Num();
	int IndicesIndex = AddElements(NumNewIndices, FGeometryCollection::FacesGroup);
	for (int32 tdx = 0; tdx < NumNewIndices; tdx++)
	{
		Indices[IndicesIndex + tdx] = FIntVector(VerticesIndex, VerticesIndex, VerticesIndex) + ElementIndices[tdx];
		Visible[IndicesIndex + tdx] = ElementVisible[tdx];
		Internal[IndicesIndex + tdx] = ElementInternal[tdx];
		MaterialIndex[IndicesIndex + tdx] = ElementMaterialIndex[tdx];
		// MaterialIDs need to be incremented
		MaterialID[IndicesIndex + tdx] = MaterialIDOffset + ElementMaterialID[tdx];	
	}

	// --- GEOMETRY GROUP ---

	int NumNewGeometryGroups = Element.NumElements(FGeometryCollection::GeometryGroup);
	NumNewGeometryGroups = (NumNewGeometryGroups == 0) ? 1 : NumNewGeometryGroups; // add one if Element input failed to create a geometry group
	int GeometryIndex = AddElements(NumNewGeometryGroups, FGeometryCollection::GeometryGroup);
	if (ElementTransformIndex.Num() > 0)
	{
		for (int32 tdx = 0; tdx < NumNewGeometryGroups; tdx++)
		{
			BoundingBox[GeometryIndex + tdx] = ElementBoundingBox[tdx];
			InnerRadius[GeometryIndex + tdx] = ElementInnerRadius[tdx];
			OuterRadius[GeometryIndex + tdx] = ElementOuterRadius[tdx];
			FaceStart[GeometryIndex + tdx] = NumIndices + ElementFaceStart[tdx];
			FaceCount[GeometryIndex + tdx] = ElementFaceCount[tdx];
			VertexStart[GeometryIndex + tdx] = NumVertices + ElementVertexStart[tdx];
			VertexCount[GeometryIndex + tdx] = ElementVertexCount[tdx];
			TransformIndex[GeometryIndex + tdx] = BoneMaps[VertexStart[GeometryIndex + tdx]];
			TransformToGeometryIndex[TransformIndex[GeometryIndex + tdx]] = GeometryIndex + tdx;
		}
	}
	else // Element input failed to create a geometry group
	{
		// Compute BoundingBox 
		BoundingBox[GeometryIndex] = FBox(ForceInitToZero);
		TransformIndex[GeometryIndex] = BoneMaps[VerticesIndex];
		VertexStart[GeometryIndex] = VerticesIndex;
		VertexCount[GeometryIndex] = NumNewVertices;
		FaceStart[GeometryIndex] = IndicesIndex;
		FaceCount[GeometryIndex] = NumNewIndices;

		TransformToGeometryIndex[TransformIndex[GeometryIndex]] = GeometryIndex;

		// Bounding Box
		for (int vdx = VerticesIndex; vdx < VerticesIndex+NumNewVertices; vdx++)
		{
			BoundingBox[GeometryIndex] += FVector(Vertices[vdx]);
		}

		// Find average particle
		// @todo (CenterOfMass) : This need to be the center of mass instead
		FVector3f Center(0);
		for (int vdx = VerticesIndex; vdx <  VerticesIndex + NumNewVertices; vdx++)
		{
			Center += Vertices[vdx];
		}

		if(NumNewVertices)
		{
			Center /= static_cast<float>(NumNewVertices);
		}

		//
		//  Inner/Outer Radius
		//
		{
			TManagedArray<float>& InnerR = InnerRadius;
			TManagedArray<float>& OuterR = OuterRadius;

			// init the radius arrays
			InnerR[GeometryIndex] = FLT_MAX;
			OuterR[GeometryIndex] = -FLT_MAX;

			// Vertices
			for (int vdx = VerticesIndex; vdx < VerticesIndex + NumNewVertices; vdx++)
			{
				float Delta = (Center - Vertices[vdx]).Size();
				InnerR[GeometryIndex] = FMath::Min(InnerR[GeometryIndex], Delta);
				OuterR[GeometryIndex] = FMath::Max(OuterR[GeometryIndex], Delta);
			}


			// Inner/Outer centroid
			for (int fdx = IndicesIndex; fdx < IndicesIndex+NumNewIndices; fdx++)
			{
				FVector3f Centroid(0);
				for (int e = 0; e < 3; e++)
				{
					Centroid += Vertices[FaceIndices[fdx][e]];
				}
				Centroid /= 3.0f;

				float Delta = (Center - Centroid).Size();
				InnerR[GeometryIndex] = FMath::Min(InnerR[GeometryIndex], Delta);
				OuterR[GeometryIndex] = FMath::Max(OuterR[GeometryIndex], Delta);
			}

			// Inner/Outer edges
			for (int fdx = IndicesIndex; fdx < IndicesIndex + NumNewIndices; fdx++)
			{
				for (int e = 0; e < 3; e++)
				{
					int i = e, j = (e + 1) % 3;
					FVector3f Edge = Vertices[FaceIndices[fdx][i]] + 0.5*(Vertices[FaceIndices[fdx][j]] - Vertices[FaceIndices[fdx][i]]);
					float Delta = (Center - Edge).Size();
					InnerR[GeometryIndex] = FMath::Min(InnerR[GeometryIndex], Delta);
					OuterR[GeometryIndex] = FMath::Max(OuterR[GeometryIndex], Delta);
				}
			}
		}
	}

	// --- MATERIAL GROUP ---
	// note for now, we rely on rebuilding mesh sections rather than passing them through.  We know
	// that MaterialID is set correctly to correspond with the material index that will be rendered
	if (ReindexAllMaterials)
	{
		ReindexMaterials();
	}	

	return StartTransformIndex;
}

bool FGeometryCollection::AppendEmbeddedInstance(int32 InExemplarIndex, int32 InParentIndex, const FTransform& InTransform)
{
	if (InParentIndex == INDEX_NONE || InParentIndex >= NumElements(FGeometryCollection::TransformGroup))
	{
		return false;
	}

	// add a new embedded instance
	int32 Element = AddElements(1, FGeometryCollection::TransformGroup);
	Transform[Element] = FTransform3f(InTransform);
	Parent[Element] = InParentIndex;
	Children[InParentIndex].Add(Element);
	SimulationType[Element] = FST_None;
	ExemplarIndex[Element] = InExemplarIndex;
	TransformToGeometryIndex[Element] = INDEX_NONE;

	return true;
}


void FGeometryCollection::ReindexExemplarIndices(TArray<int32>& SortedRemovedIndices)
{
	for (int32 Index = 0; Index < NumElements(TransformGroup); ++Index)
	{
		if (ExemplarIndex[Index] > INDEX_NONE)
		{
			for (int32 RemovalIndex = SortedRemovedIndices.Num()-1; RemovalIndex >= 0; --RemovalIndex)
			{
				if (ExemplarIndex[Index] == SortedRemovedIndices[RemovalIndex])
				{
					ExemplarIndex[Index] = INDEX_NONE;
					break;
				}
				else if (ExemplarIndex[Index] > SortedRemovedIndices[RemovalIndex])
				{
					ExemplarIndex[Index] -= (RemovalIndex + 1);
					break;
				}
			}
		}
	}
}


// Input assumes that each face has a materialID that corresponds with a render material
// This will rebuild all mesh sections
void FGeometryCollection::ReindexMaterials()
{
	FGeometryCollection::ReindexMaterials(*this);
}

void FGeometryCollection::ReindexMaterials(FManagedArrayCollection& InCollection)
{

	if (!InCollection.HasAttribute("MaterialID", FGeometryCollection::FacesGroup) ||
		!InCollection.HasAttribute("Sections", FGeometryCollection::MaterialGroup) ||
		!InCollection.HasAttribute("MaterialIndex", FGeometryCollection::FacesGroup))
	{
		return;
	}

	TManagedArray<int32>& MaterialID = InCollection.ModifyAttribute<int32>("MaterialID", FGeometryCollection::FacesGroup);
	TManagedArray<FGeometryCollectionSection>& Sections = InCollection.ModifyAttribute<FGeometryCollectionSection>("Sections", FGeometryCollection::MaterialGroup);
	TManagedArray<int32>& MaterialIndex = InCollection.ModifyAttribute<int32>("MaterialIndex", FGeometryCollection::FacesGroup);

	FManagedArrayCollection::FProcessingParameters ProcessingParams;
	ProcessingParams.bDoValidation = false; // disable validation as this can be very costly ( only in editor )

	// clear all sections	
	TArray<int32> DelSections;
	GeometryCollectionAlgo::ContiguousArray(DelSections, InCollection.NumElements(FGeometryCollection::MaterialGroup));
	InCollection.RemoveElements(FGeometryCollection::MaterialGroup, DelSections, ProcessingParams);
	DelSections.Reset(0);


	// rebuild sections		

	// count the number of triangles for each material section, adding a new section if the material ID is higher than the current number of sections
	
	for (int FaceElement = 0, nf = InCollection.NumElements(FGeometryCollection::FacesGroup); FaceElement < nf ; ++FaceElement)
	{
		int32 Section = MaterialID[FaceElement];

		while (Section + 1 > InCollection.NumElements(FGeometryCollection::MaterialGroup))
		{
			// add a new material section
			int32 Element = InCollection.AddElements(1, FGeometryCollection::MaterialGroup);

			Sections[Element].MaterialID = Element;
			Sections[Element].FirstIndex = -1;
			Sections[Element].NumTriangles = 0;
			Sections[Element].MinVertexIndex = 0;
			Sections[Element].MaxVertexIndex = 0;
		}

		Sections[Section].NumTriangles++;
	}

	// fixup the section FirstIndex and MaxVertexIndex
	for (int SectionElement = 0; SectionElement < InCollection.NumElements(FGeometryCollection::MaterialGroup); SectionElement++)
	{
		if (SectionElement == 0)
		{
			Sections[SectionElement].FirstIndex = 0;
		}
		else
		{
			// Each subsequent section has an index that starts after the last one
			// note the NumTriangles*3 - this is because indices are sent to the renderer in a flat array
			Sections[SectionElement].FirstIndex = Sections[SectionElement - 1].FirstIndex + Sections[SectionElement - 1].NumTriangles * 3;
		}

		Sections[SectionElement].MaxVertexIndex = InCollection.NumElements(FGeometryCollection::VerticesGroup) - 1;

		// if a material group no longer has any triangles in it then add material section for removal
		if (Sections[SectionElement].NumTriangles == 0)
		{
			DelSections.Push(SectionElement);
		}
	}

	// remap indices so the materials appear to be grouped
	const int32 NumSections = InCollection.NumElements(FGeometryCollection::MaterialGroup);
	const int32 NumFaceElements = InCollection.NumElements(FGeometryCollection::FacesGroup);

	// since we know the number of triangles per section we can precompute the start offset of each sections 
	// this avoid nested loop that result in N * M operations (N=sections M=faces) and we can process it in (N + M) operations instead 
	TArray<int32> OffsetPerSection;
	OffsetPerSection.AddUninitialized(NumSections);
	int32 SectionOffset = 0;
	for (int Section = 0; Section < NumSections; Section++)
	{
		OffsetPerSection[Section] = SectionOffset;
		SectionOffset += Sections[Section].NumTriangles;
	}

	for (int FaceElement = 0; FaceElement < NumFaceElements; FaceElement++)
	{
		const int32 SectionID = MaterialID[FaceElement];
		int32& SectionOffsetRef = OffsetPerSection[SectionID];
		//ensure(MaterialIndex[SectionOffsetRef] == FaceElement);
		MaterialIndex[SectionOffsetRef++] = FaceElement;
	}

	// delete unused material sections
	if (DelSections.Num())
	{
		InCollection.RemoveElements(FGeometryCollection::MaterialGroup, DelSections, ProcessingParams);
	}
}

TArray<FGeometryCollectionSection> FGeometryCollection::BuildMeshSections(const TArray<FIntVector>& InputIndices, const TArray<int32>& BaseMeshOriginalIndicesIndex, TArray<FIntVector>& RetIndices) const
{

	return FGeometryCollectionSection::BuildMeshSections(*this, InputIndices, BaseMeshOriginalIndicesIndex, RetIndices);
}


void FGeometryCollection::RemoveElements(const FName & Group, const TArray<int32>& SortedDeletionList, FProcessingParameters Params)
{
	if (SortedDeletionList.Num())
	{
		GeometryCollectionAlgo::ValidateSortedList(SortedDeletionList, NumElements(Group));

		if (Group == FTransformCollection::TransformGroup)
		{
			// Find geometry connected to the transform
			TArray<int32> GeometryIndices;
			const TManagedArray<int32>& GeometryToTransformIndex = TransformIndex;
			for (int i = 0; i < GeometryToTransformIndex.Num(); i++)
			{
				if (SortedDeletionList.Contains(GeometryToTransformIndex[i]))
				{
					GeometryIndices.Add(i);
				}
			}

			RemoveGeometryElements(GeometryIndices);

			// Find convex hulls connected to transform
			FGeometryCollectionConvexUtility::RemoveConvexHulls(this, SortedDeletionList);

			Super::RemoveElements(Group, SortedDeletionList);
		}
		else if (Group == FGeometryCollection::GeometryGroup)
		{
			RemoveGeometryElements(SortedDeletionList);
		}
		else if( Group == FGeometryCollection::FacesGroup)
		{
			BuildFaceToGeometryMapping();
			Super::RemoveElements(Group, SortedDeletionList);
			UpdateFaceGroupElements();
		}
		else if (Group == FGeometryCollection::VerticesGroup)
		{
			BuildVertexToGeometryMapping();
			Super::RemoveElements(Group, SortedDeletionList);
			UpdateVerticesGroupElements();
		}
		else
		{
			Super::RemoveElements(Group, SortedDeletionList);
		}


#if WITH_EDITOR
		if (Params.bDoValidation)
		{
			ensure(HasContiguousFaces());
			ensure(HasContiguousVertices());
			ensure(GeometryCollectionAlgo::HasValidGeometryReferences(this));
		}
#endif
	}
}

void FGeometryCollection::RemoveGeometryElements(const TArray<int32>& SortedGeometryIndicesToDelete)
{
	if (SortedGeometryIndicesToDelete.Num())
	{
		GeometryCollectionAlgo::ValidateSortedList(SortedGeometryIndicesToDelete, NumElements(FGeometryCollection::GeometryGroup));

		//
		// Find transform connected to the geometry [But don't delete them]
		//
		TArray<int32> TransformIndices;
		for (int i = 0; i < SortedGeometryIndicesToDelete.Num(); i++)
		{
			int32 GeometryIndex = SortedGeometryIndicesToDelete[i];
			if (0 <= GeometryIndex && GeometryIndex < TransformIndex.Num() && TransformIndex[GeometryIndex] != INDEX_NONE)
			{
				TransformIndices.Add(TransformIndex[GeometryIndex]);
			}
		}

		TArray<bool> Mask;

		//
		// Delete Vertices
		//
		GeometryCollectionAlgo::BuildLookupMask(TransformIndices, NumElements(FGeometryCollection::TransformGroup), Mask);

		TArray<int32> DelVertices;
		for (int32 Index = 0; Index < BoneMap.Num(); Index++)
		{
			if (BoneMap[Index] != Invalid && BoneMap[Index] < Mask.Num() && Mask[BoneMap[Index]])
			{
				DelVertices.Add(Index);
			}
		}
		DelVertices.Sort();


		//
		// Delete Faces
		//
		GeometryCollectionAlgo::BuildLookupMask(DelVertices, NumElements(FGeometryCollection::VerticesGroup), Mask);
		TManagedArray<FIntVector>& Tris = Indices;

		TArray<int32> DelFaces;
		for (int32 Index = 0; Index < Tris.Num(); Index++)
		{
			const FIntVector & Face = Tris[Index];
			for (int i = 0; i < 3; i++)
			{
				ensure(Face[i] < Mask.Num());
				if (Mask[Face[i]])
				{
					DelFaces.Add(Index);
					break;
				}
			}
		}
		DelFaces.Sort();

		Super::RemoveElements(FGeometryCollection::GeometryGroup, SortedGeometryIndicesToDelete);
		Super::RemoveElements(FGeometryCollection::VerticesGroup, DelVertices);
		Super::RemoveElements(FGeometryCollection::FacesGroup, DelFaces);

		for (int32 DeleteIdx = SortedGeometryIndicesToDelete.Num()-1; DeleteIdx >=0; DeleteIdx--)
		{
			int32 GeoIndex = SortedGeometryIndicesToDelete[DeleteIdx];
			for (int Idx = 0; Idx < TransformToGeometryIndex.Num(); Idx++)
			{
				if (TransformToGeometryIndex[Idx] > GeoIndex)
				{
					TransformToGeometryIndex[Idx]--;
				}
				else if (TransformToGeometryIndex[Idx] == GeoIndex)
				{
					TransformToGeometryIndex[Idx] = INDEX_NONE;
				}
			}

		}

		ReindexMaterials();
	}
}

void FGeometryCollection::Empty()
{
	for (const FName& GroupName : GroupNames())
	{
		EmptyGroup(GroupName);
	}
	// re-initialize interfaces
	InitializeInterfaces();
	SetNumUVLayers(1);
}

void FGeometryCollection::Reset()
{
	Super::Reset();
	Construct();
}

void FGeometryCollection::InitializeInterfaces()
{
	FGeometryCollectionConvexPropertiesInterface::InitializeInterface();
	FGeometryCollectionProximityPropertiesInterface::InitializeInterface();
}

void FGeometryCollection::ReorderElements(FName Group, const TArray<int32>& NewOrder)
{
	if (Group == FTransformCollection::TransformGroup)
	{
		ReorderTransformElements(NewOrder);
	}
	else if (Group == FGeometryCollection::GeometryGroup)
	{
		ReorderGeometryElements(NewOrder);
	}
	else
	{
		Super::ReorderElements(Group, NewOrder);
	}
}

void FGeometryCollection::ReorderTransformElements(const TArray<int32>& NewOrder)
{
	struct FTransformGeomPair
	{
		FTransformGeomPair(int32 InTransformIdx, int32 InGeomIdx) : TransformIdx(InTransformIdx), GeomIdx(InGeomIdx) {}

		int32 TransformIdx;
		int32 GeomIdx;

		bool operator<(const FTransformGeomPair& Other) const { return TransformIdx < Other.TransformIdx; }
	};

	const int32 NumGeometries = TransformIndex.Num();
	TArray<FTransformGeomPair> Pairs;
	Pairs.Reserve(NumGeometries);
	for (int32 GeomIdx = 0; GeomIdx < NumGeometries; ++GeomIdx)
	{
		if (TransformIndex[GeomIdx] != INDEX_NONE)
		{
			Pairs.Emplace(NewOrder[TransformIndex[GeomIdx]], GeomIdx);
		}
	}
	Pairs.Sort();

	TArray<int32> NewGeomOrder;
	NewGeomOrder.Reserve(NumGeometries);
	for (const FTransformGeomPair& Pair : Pairs)
	{
		NewGeomOrder.Add(Pair.GeomIdx);
	}
	ReorderGeometryElements(NewGeomOrder);

	for (int32 Index = 0; Index < Parent.Num(); Index++)
	{
		// remap the parents (-1 === Invalid )
		if (Parent[Index] != -1)
		{
			Parent[Index] = NewOrder[Parent[Index]];
		}

		// remap children
		TSet<int32> ChildrenCopy = Children[Index];
		Children[Index].Empty();
		for (int32 ChildID : ChildrenCopy)
		{
			if (ChildID >= 0)
			{
				Children[Index].Add(NewOrder[ChildID]);
			}
			else
			{
				Children[Index].Add(ChildID);	//not remap so just leave as was
			}
		}
	}

	Super::ReorderElements(FTransformCollection::TransformGroup, NewOrder);

}

void FGeometryCollection::ReorderGeometryElements(const TArray<int32>& NewOrder)
{
	const int32 NumGeometry = NumElements(GeometryGroup);
	check(NumGeometry == NewOrder.Num());

	//Compute new order for vertices group and faces group
	TArray<int32> NewVertOrder;
	NewVertOrder.Reserve(NumElements(VerticesGroup));
	int32 TotalVertOffset = 0;

	TArray<int32> NewFaceOrder;
	NewFaceOrder.Reserve(NumElements(FacesGroup));
	int32 TotalFaceOffset = 0;

	for (int32 OldGeomIdx = 0; OldGeomIdx < NumGeometry; ++OldGeomIdx)
	{
		const int32 NewGeomIdx = NewOrder[OldGeomIdx];

		//verts
		const int32 VertStartIdx = VertexStart[NewGeomIdx];
		const int32 NumVerts = VertexCount[NewGeomIdx];
		for (int32 VertIdx = VertStartIdx; VertIdx < VertStartIdx + NumVerts; ++VertIdx)
		{
			NewVertOrder.Add(VertIdx);
		}
		TotalVertOffset += NumVerts;

		//faces
		const int32 FaceStartIdx = FaceStart[NewGeomIdx];
		const int32 NumFaces = FaceCount[NewGeomIdx];
		for (int32 FaceIdx = FaceStartIdx; FaceIdx < FaceStartIdx + NumFaces; ++FaceIdx)
		{
			NewFaceOrder.Add(FaceIdx);
		}
		TotalFaceOffset += NumFaces;
	}

	//we must now reorder according to dependencies
	Super::ReorderElements(VerticesGroup, NewVertOrder);
	Super::ReorderElements(FacesGroup, NewFaceOrder);
	Super::ReorderElements(GeometryGroup, NewOrder);
}

bool FGeometryCollection::BuildVertexToGeometryMapping(bool InSaved )
{
	bool AttributeAdded = false;

	if (!FindAttribute<int32>("VertexToGeometryIndex", VerticesGroup))
	{
		FConstructionParameters NotSaved = { FName(), InSaved };
		AddAttribute<int32>("VertexToGeometryIndex", VerticesGroup, NotSaved);
		AttributeAdded = true;
	}

	TManagedArray<int32>* VertexGeometryMap = FindAttribute<int32>("VertexToGeometryIndex", VerticesGroup);
	if (ensure(VertexGeometryMap))
	{
		for (int32 GeometryIndex = NumElements(GeometryGroup) - 1; GeometryIndex >= 0; GeometryIndex--)
		{
			int VertexEnd = VertexStart[GeometryIndex] + VertexCount[GeometryIndex];
			for (int32 VertexIndex = VertexStart[GeometryIndex]; VertexIndex < VertexEnd; VertexIndex++)
			{
				(*VertexGeometryMap)[VertexIndex] = GeometryIndex;
			}
		}
	}
	return AttributeAdded;
}

void FGeometryCollection::UpdateVerticesGroupElements()
{
	//
	//  Reset the VertexCount array
	//
	TManagedArray<int32>* VertexGeometryMap = FindAttribute<int32>("VertexToGeometryIndex", VerticesGroup);
	if(ensure(VertexGeometryMap))
	{ 
		const int32 NumGeometryElements = NumElements(FGeometryCollection::GeometryGroup);
		for (int32 GeometryIndex = NumGeometryElements - 1; GeometryIndex >= 0; GeometryIndex--)
		{
			VertexStart[GeometryIndex] = INT_MAX;
			VertexCount[GeometryIndex] = 0;
		}

		const int32 NumVertexElements = NumElements(VerticesGroup);
		for (int32 VertexIndex = NumVertexElements - 1; VertexIndex >= 0; VertexIndex--)
		{
			VertexStart[ (*VertexGeometryMap)[VertexIndex] ] = FMath::Min(VertexStart[ (*VertexGeometryMap)[VertexIndex] ], VertexIndex);
			VertexCount[ (*VertexGeometryMap)[VertexIndex] ]++;
		}

		// final pass through to fix any groups with zero vertices, which will not have had their start/count updated by the above loop
		// note: technically if count is zero, start does not matter, but other code assumes the starts are in monotonic order
		int32 LastVertexStart = 0, LastVertexCount = 0;
		for (int32 GeometryIndex = 0; GeometryIndex < NumGeometryElements; ++GeometryIndex)
		{
			if (VertexCount[GeometryIndex] == 0)
			{
				VertexStart[GeometryIndex] = LastVertexStart + LastVertexCount;
			}
			else
			{
				LastVertexStart = VertexStart[GeometryIndex];
				LastVertexCount = VertexCount[GeometryIndex];
			}
		}
	}

	ensure(HasContiguousVertices());
}

bool FGeometryCollection::BuildFaceToGeometryMapping(bool InSaved)
{
	bool AttributeAdded = false;

	if (!FindAttribute<int32>("FaceToGeometryIndex", FacesGroup))
	{
		FConstructionParameters NotSaved = { FName(), InSaved };
		AddAttribute<int32>("FaceToGeometryIndex", FacesGroup, NotSaved);
		AttributeAdded = true;
	}

	TManagedArray<int32>* FaceGeometryMap = FindAttribute<int32>("FaceToGeometryIndex", FacesGroup);
	if (ensure(FaceGeometryMap))
	{
		for (int32 GeometryIndex = NumElements(GeometryGroup) - 1; GeometryIndex >= 0; GeometryIndex--)
		{
			int FaceEnd = FaceStart[GeometryIndex] + FaceCount[GeometryIndex];
			for (int32 FaceIndex = FaceStart[GeometryIndex]; FaceIndex < FaceEnd; FaceIndex++)
			{
				(*FaceGeometryMap)[FaceIndex] = GeometryIndex;
			}
		}
	}
	return AttributeAdded;
}

void FGeometryCollection::UpdateFaceGroupElements()
{
	//
	//  Reset the FaceCount and NumFaces array
	//

	TManagedArray<int32>* FaceGeometryMap = FindAttribute<int32>("FaceToGeometryIndex", FacesGroup);
	if (ensure(FaceGeometryMap))
	{
		const int32 NumGeometryElements = NumElements(FGeometryCollection::GeometryGroup);
		for (int32 GeometryIndex = NumGeometryElements - 1; GeometryIndex >= 0; GeometryIndex--)
		{
			FaceStart[GeometryIndex] = INT_MAX;
			FaceCount[GeometryIndex] = 0;
		}

		const int32 NumFaceElements = NumElements(FacesGroup);
		for (int32 FaceIndex = NumFaceElements - 1; FaceIndex >= 0; FaceIndex--)
		{
			FaceStart[(*FaceGeometryMap)[FaceIndex]] = FMath::Min(FaceStart[(*FaceGeometryMap)[FaceIndex]], FaceIndex);
			FaceCount[(*FaceGeometryMap)[FaceIndex]]++;
		}

		// final pass through to fix any groups with zero faces, which will not have had their start/count updated by the above loop
		// note: technically if count is zero, start does not matter, but other code assumes the starts are in monotonic order
		int32 LastFaceStart = 0, LastFaceCount = 0;
		for (int32 GeometryIndex = 0; GeometryIndex < NumGeometryElements; ++GeometryIndex)
		{
			if (FaceCount[GeometryIndex] == 0)
			{
				FaceStart[GeometryIndex] = LastFaceStart + LastFaceCount;
			}
			else
			{
				LastFaceStart = FaceStart[GeometryIndex];
				LastFaceCount = FaceCount[GeometryIndex];
			}
		}
	}

	ensure(HasContiguousVertices());
}


void FGeometryCollection::UpdateGeometryVisibility(const TArray<int32>& NodeList, bool VisibilityState)
{

	for (int32 Idx = 0; Idx < Visible.Num(); Idx++)
	{
		for (int32 Node : NodeList)
		{
			if (BoneMap[Indices[Idx][0]] == Node)
			{
				Visible[Idx] = VisibilityState;
			}
		}	
	}
}
bool FGeometryCollection::HasVisibleGeometry() const
{
	bool bHasVisibleGeometry = false;
	const TManagedArray<bool>& VisibleIndices = Visible;

	for (int32 fdx = 0; fdx < VisibleIndices.Num(); fdx++)
	{
		if (VisibleIndices[fdx])
		{
			bHasVisibleGeometry = true;
			break;
		}
	}
	return bHasVisibleGeometry;
}

void FGeometryCollection::UpdateBoundingBox()
{
	FGeometryCollection::UpdateBoundingBox(*this, /*bSkipCheck*/ true);
}

void FGeometryCollection::UpdateBoundingBox(FManagedArrayCollection& InCollection, bool bSkipCheck)
{
	if (!bSkipCheck && (
		!InCollection.HasAttribute("BoundingBox", FGeometryCollection::GeometryGroup) ||
		!InCollection.HasAttribute("Vertex", FGeometryCollection::VerticesGroup) ||
		!InCollection.HasAttribute("BoneMap", FGeometryCollection::VerticesGroup) ||
		!InCollection.HasAttribute("TransformToGeometryIndex", FTransformCollection::TransformGroup))) 
	{
		return;
	}

	TManagedArray<FBox>& BoundingBox = InCollection.ModifyAttribute<FBox>("BoundingBox", FGeometryCollection::GeometryGroup);
	const TManagedArray<FVector3f>& Vertex = InCollection.GetAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup);
	const TManagedArray<int32>& BoneMap = InCollection.GetAttribute<int32>("BoneMap", FGeometryCollection::VerticesGroup);
	const TManagedArray<int32>& TransformToGeometryIndex = InCollection.GetAttribute<int32>("TransformToGeometryIndex", FTransformCollection::TransformGroup);

	if (BoundingBox.Num())
	{
		// Initialize BoundingBox
		for (int32 Idx = 0; Idx < BoundingBox.Num(); ++Idx)
		{
			BoundingBox[Idx].Init();
		}

		// Compute BoundingBox
		for (int32 Idx = 0; Idx < Vertex.Num(); ++Idx)
		{
			const int32 TransformIndexValue = BoneMap[Idx];
			if (TransformIndexValue != INDEX_NONE)
			{
				const int32 GeometryIndex = TransformToGeometryIndex[TransformIndexValue];
				if (GeometryIndex != INDEX_NONE)
				{
					BoundingBox[GeometryIndex] += FVector(Vertex[Idx]);
				}
			}
		}
	}
}


FBoxSphereBounds FGeometryCollection::GetBoundingBox() const 
{
	TArray<FTransform> GlobalTransformArray;
	GeometryCollectionAlgo::GlobalMatrices(Transform, Parent, GlobalTransformArray);
	FBox CombinedBounds(EForceInit::ForceInit);
	for (int32 GeoIdx = 0; GeoIdx < NumElements(FGeometryCollection::GeometryGroup); ++GeoIdx)
	{
		int32 TransformIdx = TransformIndex[GeoIdx];
		CombinedBounds += BoundingBox[GeoIdx].TransformBy(GlobalTransformArray[TransformIdx]);
	}
	FBoxSphereBounds CombinedBoxSphereBounds(CombinedBounds);
	return CombinedBoxSphereBounds;
}


void FGeometryCollection::Serialize(Chaos::FChaosArchive& Ar)
{
	if (Ar.IsCooking())
	{
		FGeometryCollectionConvexPropertiesInterface::CleanInterfaceForCook();
		FGeometryCollectionProximityPropertiesInterface::CleanInterfaceForCook();
	}

	Ar.UsingCustomVersion(FFortniteSeasonBranchObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	Super::Serialize(Ar);

	if (Ar.IsLoading())
	{
		// Versioning - correct assets that were saved before material sections were introduced
		if (NumElements(FGeometryCollection::MaterialGroup) == 0)
		{
			int SectionIndex = AddElements(1, FGeometryCollection::MaterialGroup);
			Sections[SectionIndex].MaterialID = 0;
			Sections[SectionIndex].FirstIndex = 0;
			Sections[SectionIndex].NumTriangles = Indices.Num();
			Sections[SectionIndex].MinVertexIndex = 0;
			Sections[SectionIndex].MaxVertexIndex = Vertex.Num();
		}

		// Recompute TransformToGroupIndex map
		int NumGeoms = NumElements(FGeometryCollection::GeometryGroup);
		int NumTransforms = NumElements(FGeometryCollection::TransformGroup);
		for (int32 Idx = 0; Idx < NumGeoms; ++Idx)
		{
			if (0<=TransformIndex[Idx]&&TransformIndex[Idx] < NumTransforms)
			{
				TransformToGeometryIndex[TransformIndex[Idx]] = Idx;
			}
		}

		// Add SimulationType attribute
		if (!(this->HasAttribute(FGeometryCollection::SimulationTypeAttribute, FTransformCollection::TransformGroup)))
		{
			TManagedArray<int32>& SimType = this->AddAttribute<int32>(FGeometryCollection::SimulationTypeAttribute, FTransformCollection::TransformGroup);
			for (int32 Idx = 0; Idx < NumTransforms; ++Idx)
			{
				SimType[Idx] = FGeometryCollection::ESimulationTypes::FST_None;
			}
		}

		// for backwards compatibility convert old BoneHierarchy struct into split out arrays
		enum ENodeFlags : uint32
		{
			// additional flags
			FS_Clustered = 0x00000002,
		};

		const TManagedArray<FGeometryCollectionBoneNode>* BoneHierarchyPtr = FindAttribute<FGeometryCollectionBoneNode>("BoneHierarchy", FTransformCollection::TransformGroup);
		if (BoneHierarchyPtr)
		{
			const TManagedArray<FGeometryCollectionBoneNode>& BoneHierarchy = *BoneHierarchyPtr;

			for (int Idx = 0; Idx < BoneHierarchy.Num(); Idx++)
			{
				if (!HasAttribute("Level", FGeometryCollection::TransformGroup))
				{
					AddAttribute<int32>("Level", FGeometryCollection::TransformGroup);
				}
				TManagedArray<int32>& Level = ModifyAttribute<int32>("Level", FGeometryCollection::TransformGroup);
				Level[Idx] = BoneHierarchy[Idx].Level;

				SimulationType[Idx] = ESimulationTypes::FST_Rigid;
				StatusFlags[Idx] = FGeometryCollection::ENodeFlags::FS_None;

				if (BoneHierarchy[Idx].StatusFlags & ENodeFlags::FS_Clustered)
				{
					SimulationType[Idx] = ESimulationTypes::FST_Clustered;
				}
				if (BoneHierarchy[Idx].StatusFlags & FGeometryCollection::ENodeFlags::FS_RemoveOnFracture)
				{
					StatusFlags[Idx] |= FGeometryCollection::ENodeFlags::FS_RemoveOnFracture;
				}
			}
		}



		// Version 5 introduced accurate SimulationType tagging
		if (Version < 5)
		{
			UE_LOGF(FGeometryCollectionLogging, Log, "GeometryCollection has inaccurate simulation type tags. Updating tags based on transform topology.");
			const TManagedArray<bool>* SimulatableParticles = FindAttribute<bool>(FGeometryCollection::SimulatableParticlesAttribute, FTransformCollection::TransformGroup);
			TArray<bool> RigidChildren; RigidChildren.Init(false,NumElements(FTransformCollection::TransformGroup));
			const TArray<int32> RecursiveOrder = GeometryCollectionAlgo::ComputeRecursiveOrder(*this);
			for (const int32 TransformGroupIndex : RecursiveOrder)
			{
				SimulationType[TransformGroupIndex] = ESimulationTypes::FST_None;

				if(!Children[TransformGroupIndex].Num())
				{ // leaf nodes
					if (TransformToGeometryIndex[TransformGroupIndex] > INDEX_NONE)
					{
						SimulationType[TransformGroupIndex] = ESimulationTypes::FST_Rigid;
					}

					if (SimulatableParticles && !(*SimulatableParticles)[TransformGroupIndex])
					{
						SimulationType[TransformGroupIndex] = ESimulationTypes::FST_None;
					}
				}
				else
				{ // interior nodes
					if (RigidChildren[TransformGroupIndex])
					{
						SimulationType[TransformGroupIndex] = ESimulationTypes::FST_Clustered;
					}
					else if (TransformToGeometryIndex[TransformGroupIndex] > INDEX_NONE)
					{
						SimulationType[TransformGroupIndex] = ESimulationTypes::FST_Rigid;
					}
				}

				if (SimulationType[TransformGroupIndex] != ESimulationTypes::FST_None &&
					Parent[TransformGroupIndex] != INDEX_NONE )
				{
					RigidChildren[Parent[TransformGroupIndex]] = true;
				}
			}

			// Structure is conditioned, now considered up to date.
			Version = 5;
		}

		// Version 6 introduced the Exemplar Index array
		if (Version < 6)
		{
			ExemplarIndex.Fill(INDEX_NONE);

			// Structure is conditioned, now considered up to date.
			Version = 6;
		}

		if (Version < 7)
		{
			if (HasAttribute("TransformToConvexIndex", FTransformCollection::TransformGroup))
			{
				TManagedArray<int32> TransformToConvexIndex = MoveTemp(ModifyAttribute<int32>("TransformToConvexIndex", FTransformCollection::TransformGroup));
				RemoveAttribute("TransformToConvexIndex", FTransformCollection::TransformGroup);
				// if we don't already have the one-to-many version, convert the previous one-to-one mapping to the new format
				if (!HasAttribute("TransformToConvexIndices", FTransformCollection::TransformGroup))
				{
					FManagedArrayCollection::FConstructionParameters ConvexDependency(FGeometryCollection::ConvexGroup);
					TManagedArray<TSet<int32>>& IndexSets = AddAttribute<TSet<int32>>("TransformToConvexIndices", FTransformCollection::TransformGroup, ConvexDependency);
					for (int32 TransformIdx = 0; TransformIdx < TransformToConvexIndex.Num(); TransformIdx++)
					{
						int32 ConvexIdx = TransformToConvexIndex[TransformIdx];
						if (ConvexIdx != INDEX_NONE)
						{
							IndexSets[TransformIdx].Add(ConvexIdx);
						}
					}
				}
			}
			Version = 7;
		}

		// Version 8 introduced multiple UVs.
		if (Version < 8)
		{
			if (!HasAttribute("UVs", FGeometryCollection::VerticesGroup))
			{
				// Note: As UVs is an external attribute that is always added by Construct, this should never be encountered
				UE_LOGF(FGeometryCollectionLogging, Log, "GeometryCollection updated to multiple UV sets.");
				AddAttribute<TArray<FVector2f>>("UVs", FGeometryCollection::VerticesGroup);
			}

			TManagedArray<TArray<FVector2f>>& MultipleUVs = ModifyAttribute<TArray<FVector2f>>("UVs", FGeometryCollection::VerticesGroup);
			int32 MinUVLayers = 8;
			for (int32 VertIdx = 0; VertIdx < MultipleUVs.Num(); ++VertIdx)
			{
				MinUVLayers = FMath::Min(MultipleUVs[VertIdx].Num(), MinUVLayers);
			}
			if (MinUVLayers < 1)
			{
				for (int32 VertIdx = 0; VertIdx < MultipleUVs.Num(); ++VertIdx)
				{
					MultipleUVs[VertIdx].SetNum(1);
				}
			}

			if (const TManagedArray<FVector2f>* SingleUV = FindAttribute<FVector2f>("UV", FGeometryCollection::VerticesGroup))
			{
				for (int32 VertIdx = 0; VertIdx < MultipleUVs.Num(); ++VertIdx)
				{
					if (SingleUV)
					{
						MultipleUVs[VertIdx][0] = (*SingleUV)[VertIdx];
					}
				}

				RemoveAttribute("UV", FGeometryCollection::VerticesGroup);
			}

			// Structure is conditioned, now considered up to date.
			Version = 8;
		}

		// Version 9 fixed fully-invisible geometry and invalid exemplars left in the hierarchy (artifacts from old fracture + the Version 5 change)
		if (Version < 9)
		{
			auto HasVisibleFaces = [this](int32 TransformGroupIndex) -> bool
			{
				int32 GeometryIndex = TransformToGeometryIndex[TransformGroupIndex];
				if (GeometryIndex == INDEX_NONE)
				{
					return false;
				}
				int32 Start = FaceStart[GeometryIndex], Count = FaceCount[GeometryIndex];
				for (int32 FaceIndex = Start; FaceIndex < Start + Count; FaceIndex++)
				{
					if (Visible[FaceIndex])
					{
						return true;
					}
				}
				return false;
			};
		
			TArray<int32> InvalidTransforms, InvalidGeometry;
			TArray<bool> RigidChildren; RigidChildren.Init(false, NumElements(FTransformCollection::TransformGroup));
			const TArray<int32> RecursiveOrder = GeometryCollectionAlgo::ComputeRecursiveOrder(*this);
			for (const int32 TransformGroupIndex : RecursiveOrder)
			{
				bool bHasExemplar = ExemplarIndex[TransformGroupIndex] > INDEX_NONE;
				bool bHasGeometry = HasVisibleFaces(TransformGroupIndex);
				bool bHasRigidChildren = RigidChildren[TransformGroupIndex];
				bool bKeep = true;
				if (SimulationType[TransformGroupIndex] == ESimulationTypes::FST_None && !bHasExemplar) // handle exemplars with no exemplar (remove or convert to rigid)
				{
					if (bHasGeometry)
					{
						SimulationType[TransformGroupIndex] = ESimulationTypes::FST_Rigid;
					}
					else
					{
						InvalidTransforms.Add(TransformGroupIndex);
						bKeep = false;
					}
				}
				else if (SimulationType[TransformGroupIndex] == ESimulationTypes::FST_Rigid ) // handle internal rigids
				{
					if (bHasRigidChildren)
					{
						SimulationType[TransformGroupIndex] = ESimulationTypes::FST_Clustered;
					}
				}
				if (bKeep && SimulationType[TransformGroupIndex] != ESimulationTypes::FST_None && Parent[TransformGroupIndex] != INDEX_NONE)
				{
					RigidChildren[Parent[TransformGroupIndex]] = true;
				}
			}
			if (InvalidGeometry.Num() > 0)
			{
				UE_LOGF(FGeometryCollectionLogging, Log, "Removing %d invalid, fully-invisible geometries from geometry collection.", InvalidGeometry.Num());
				InvalidGeometry.Sort();
				RemoveElements(GeometryGroup, InvalidGeometry);
			}
			if (InvalidTransforms.Num() > 0)
			{
				UE_LOGF(FGeometryCollectionLogging, Log, "Removing %d invalid, empty transforms from geometry collection.", InvalidTransforms.Num());
				InvalidTransforms.Sort();
				RemoveElements(TransformGroup, InvalidTransforms);
			}
			
			Version = 9;
		}

		Chaos::Facades::FCollectionHierarchyFacade HierarchyFacade(*this);
		if (Ar.CustomVer(FFortniteSeasonBranchObjectVersion::GUID) < FFortniteSeasonBranchObjectVersion::ChaosGeometryCollectionSaveLevelsAttribute
			|| !HierarchyFacade.HasLevelAttribute()
			|| !HierarchyFacade.IsLevelAttributePersistent()
			)
		{
			// Level attribute previously serialized with bSave = false, so was not serializing level data.
			// We now compute this during cook and need to serialize, so convert attribute to bSave = true
			// this is handled by the facade 
			HierarchyFacade.GenerateLevelAttribute();
		}

		if (Version < 10)
		{
			if (!HasAttribute(GeometryCollection::UV::UVLayerNames[0], VerticesGroup) || HasAttribute("UVs", VerticesGroup))
			{
				TManagedArray<TArray<FVector2f>>* OrigUVs = FindAttributeTyped<TArray<FVector2f>>("UVs", VerticesGroup);
				if (OrigUVs)
				{
					// Note: We take the max of the num layers because in practice there have been some vertices with inconsistent layer counts
					// and it seems better to transfer all the data (with missing data left as zeros) than to potentially lose data
					int32 NumLayers = 1;
					for (int32 Idx = 0; Idx < OrigUVs->Num(); ++Idx)
					{
						NumLayers = FMath::Max((*OrigUVs)[Idx].Num(), NumLayers);
					}
					NumLayers = FMath::Min((int32)GeometryCollectionUV::MAX_NUM_UV_CHANNELS, NumLayers); // make sure we never exceed max layers
					SetNumUVLayers(NumLayers);

					for (int32 Idx = 0; Idx < OrigUVs->Num(); ++Idx)
					{
						TArray<FVector2f>& VertexLayers = (*OrigUVs)[Idx];
						int32 NumVertexLayers = FMath::Min(VertexLayers.Num(), NumLayers);
						for (int32 LayerIdx = 0; LayerIdx < NumVertexLayers; ++LayerIdx)
						{
							ModifyUV(Idx, LayerIdx) = VertexLayers[LayerIdx];
						}
						for (int32 LayerIdx = NumVertexLayers; LayerIdx < NumLayers; ++LayerIdx)
						{
							ModifyUV(Idx, LayerIdx) = FVector2f(0, 0); // explicitly zero UVs of any missing layers
						}
					}

					RemoveAttribute("UVs", VerticesGroup);
				}
				else
				{
					SetNumUVLayers(1);
				}
			}

			Version = 10;
		}

		if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::ChaosGeometryCollectionInternalFacesAttribute)
		{
			if (!HasAttribute("Internal", FacesGroup))
			{
				AddExternalAttribute<bool>("Internal", FacesGroup, Internal);
			}

			for (int32 FaceIdx = 0; FaceIdx < MaterialID.Num(); ++FaceIdx)
			{
				Internal[FaceIdx] = bool(MaterialID[FaceIdx] & 1);
			}
		}

		if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::ChaosGeometryCollectionConnectionEdgeGroup)
		{
			// Migrate old Connections TSet<int32> data to arrays of edge data
			// Note we intentionally do *not* use the facade here, as the migration is specific to how the data is at the current moment, and the facade may change w/ future data changes.
			if (TManagedArray<TSet<int32>>* Connections = FindAttribute<TSet<int32>>("Connections", TransformGroup))
			{
				const FName ConnectionGroupName = "ConnectionEdge";
				AddGroup(ConnectionGroupName);
				TManagedArray<int32>& Starts = AddAttribute<int32>("ConnectionEdgeStarts", ConnectionGroupName, FConstructionParameters(FTransformCollection::TransformGroup, true));
				TManagedArray<int32>& Ends = AddAttribute<int32>("ConnectionEdgeEnds", ConnectionGroupName, FConstructionParameters(FTransformCollection::TransformGroup, true));
				for (int32 TransformIdx = 0; TransformIdx < NumElements(TransformGroup); ++TransformIdx)
				{
					for (int32 NbrIdx : (*Connections)[TransformIdx])
					{
						if (TransformIdx < NbrIdx)
						{
							int32 EdgeIdx = AddElements(1, ConnectionGroupName);
							Starts[EdgeIdx] = TransformIdx;
							Ends[EdgeIdx] = NbrIdx;
						}
					}
				}
				RemoveAttribute("Connections", TransformGroup);
			}
		}

		// Finally, make sure expected interfaces are initialized
		InitializeInterfaces();
	}

	ensure(Version == GetLatestVersionNumber());
}

bool FGeometryCollection::HasContiguousVertices( ) const
{
	int32 NumTransforms = NumElements(FGeometryCollection::TransformGroup);

	TSet<int32> TransformIDs;
	TArray<int32> RecreatedBoneIds;
	RecreatedBoneIds.Init(-1, NumElements(FGeometryCollection::VerticesGroup));
	int32 NumTransformIndex = TransformIndex.Num();
	int32 NumBoneIndex = BoneMap.Num();
	for (int32 GeometryIndex = 0; GeometryIndex < NumTransformIndex; GeometryIndex++)
	{ // for each known geometry...
		int32 TransformIDFromGeometry = TransformIndex[GeometryIndex];
		int32 StartIndex = VertexStart[GeometryIndex];
		int32 NumVertices = VertexCount[GeometryIndex];

		if (TransformIDs.Contains(TransformIDFromGeometry))
		{
			return false;
		}
		TransformIDs.Add(TransformIDFromGeometry);

		int32 Counter = NumVertices;
		for (int32 BoneIndex = 0 ; BoneIndex < NumBoneIndex; ++BoneIndex)
		{ // for each mapping from the vertex to the transform hierarchy ... 
			if (StartIndex <= BoneIndex && BoneIndex < (StartIndex + NumVertices))
			{ // process just the specified range
				int32 TransformIDFromBoneMap = BoneMap[BoneIndex];
				RecreatedBoneIds[BoneIndex] = BoneMap[BoneIndex];
				if (TransformIDFromBoneMap < 0 || NumTransforms <= TransformIDFromBoneMap)
				{ // not contiguous if index is out of range
					return false;
				}
				if (TransformIDFromGeometry != TransformIDFromBoneMap)
				{ // not contiguous if indexing into a different transform
					return false;
				}
				--Counter;
			}
		}

		if (Counter)
		{
			return false;
		}
	}
	for (int32 Index = 0; Index < NumElements(FGeometryCollection::VerticesGroup); ++Index)
	{
		if (RecreatedBoneIds[Index] < 0)
		{
			return false;
		}
	}

	return true;
}


bool FGeometryCollection::HasContiguousFaces() const
{
	int32 TotalNumTransforms = NumElements(FGeometryCollection::TransformGroup);

	// vertices
	int32 TotalNumVertices = NumElements(FGeometryCollection::VerticesGroup);

	int32 NumIndices = Indices.Num();
	int32 NumTransformIndex = TransformIndex.Num();
	for (int32 GeometryIndex = 0; GeometryIndex < NumTransformIndex ; ++GeometryIndex)
	{ // for each known geometry...
		int32 TransformIDFromGeometry = TransformIndex[GeometryIndex];
		int32 StartIndex = FaceStart[GeometryIndex];
		int32 NumFaces = FaceCount[GeometryIndex];

		int32 Counter = NumFaces;
		for (int32 FaceIndex = 0 ; FaceIndex < NumIndices; ++FaceIndex)
		{ // for each mapping from the vertex to the transform hierarchy ... 
			if (StartIndex <= FaceIndex && FaceIndex < (StartIndex + NumFaces))
			{ // process just the specified range
				for (int32 i = 0; i < 3; ++i)
				{
					int32 VertexIndex = Indices[FaceIndex][i];
					if (VertexIndex < 0 || TotalNumVertices <= VertexIndex)
					{
						return false;
					}

					int32 TransformIDFromBoneMap = BoneMap[VertexIndex];

					if (TransformIDFromBoneMap < 0 && TotalNumTransforms < TransformIDFromBoneMap)
					{ // not contiguous if index is out of range
						return false;
					}
					if (TransformIDFromGeometry != TransformIDFromBoneMap)
					{ // not contiguous if indexing into a different transform
						return false;
					}
				}
				--Counter;
			}
		}

		if (Counter)
		{
			return false;
		}
	}
	return true;
}

bool FGeometryCollection::HasContiguousRenderFaces() const
{
	// validate all remapped indexes have their materials ID's grouped an in increasing order
	int LastMaterialID = 0;
	for (int32 IndexIdx = 0, NumElementsFaceGroup = NumElements(FGeometryCollection::FacesGroup); IndexIdx < NumElementsFaceGroup ; ++IndexIdx)
	{
		if (LastMaterialID > MaterialID[MaterialIndex[IndexIdx]])
			return false;
		LastMaterialID = MaterialID[MaterialIndex[IndexIdx]];
	}

	// check sections ranges do all point to a single material
	for (int32 MaterialIdx = 0, NumElementsMaterialGroup = NumElements(FGeometryCollection::MaterialGroup) ; MaterialIdx < NumElementsMaterialGroup ; ++MaterialIdx)
	{
		int first = Sections[MaterialIdx].FirstIndex / 3;
		int last = first + Sections[MaterialIdx].NumTriangles;

		for (int32 IndexIdx = first; IndexIdx < last; ++IndexIdx)
		{
			if ( (MaterialID[MaterialIndex[IndexIdx]]) != MaterialIdx )
				return false;
		}

	}

	return true;
}

int32 FGeometryCollection::NumUVLayers() const
{
	return GeometryCollection::UV::GetNumUVLayers(*this);
}

bool FGeometryCollection::SetNumUVLayers(int32 NumLayers)
{
	return GeometryCollection::UV::SetNumUVLayers(*this, NumLayers);
}

bool FGeometryCollection::IsVisible(int32 Element) const
{
	if (!IsRigid(Element))
	{
		return false;
	}

	if (TransformToGeometryIndex[Element] > INDEX_NONE)
	{
		int32 CurrFace = FaceStart[TransformToGeometryIndex[Element]];
		for (int32 FaceOffset = 0; FaceOffset < FaceCount[TransformToGeometryIndex[Element]]; ++FaceOffset)
		{
			if (Visible[CurrFace + FaceOffset])
			{
				return true;
			}
		}
	}

	return false;;
}

FGeometryCollection* FGeometryCollection::NewGeometryCollection(const TArray<float>& RawVertexArray, const TArray<int32>& RawIndicesArray, bool ReverseVertexOrder, const FGeometryCollectionDefaults RawDefaults)
{
	FGeometryCollection* Collection = new FGeometryCollection(RawDefaults);
	FGeometryCollection::Init(Collection, RawVertexArray, RawIndicesArray, ReverseVertexOrder);
	return Collection;
}

void FGeometryCollection::Init(FGeometryCollection* Collection, const TArray<float>& RawVertexArray, const TArray<int32>& RawIndicesArray, bool ReverseVertexOrder)
{
	if (Collection)
	{
		int NumNewVertices = RawVertexArray.Num() / 3;
		int VerticesIndex = Collection->AddElements(NumNewVertices, FGeometryCollection::VerticesGroup);

		int NumNewIndices = RawIndicesArray.Num() / 3;
		int IndicesIndex = Collection->AddElements(NumNewIndices, FGeometryCollection::FacesGroup);

		int NumNewParticles = 1; // 1 particle for this geometry structure
		int ParticlesIndex = Collection->AddElements(NumNewParticles, FGeometryCollection::TransformGroup);

		TManagedArray<FVector3f>& Vertices = Collection->Vertex;
		TManagedArray<FVector3f>& Normals = Collection->Normal;
		TManagedArray<FVector3f>& TangentU = Collection->TangentU;
		TManagedArray<FVector3f>& TangentV = Collection->TangentV;
		TManagedArray<FLinearColor>& Colors = Collection->Color;
		TManagedArray<FIntVector>& Indices = Collection->Indices;
		TManagedArray<bool>& Visible = Collection->Visible;
		TManagedArray<int32>& MaterialID = Collection->MaterialID;
		TManagedArray<int32>& MaterialIndex = Collection->MaterialIndex;
		TManagedArray<bool>& Internal = Collection->Internal;
		TManagedArray<FTransform3f>& Transform = Collection->Transform;
		TManagedArray<int32>& BoneMap = Collection->BoneMap;
		
		Collection->SetNumUVLayers(1);

		// set the vertex information
		TManagedArray<FVector2f>* UV0 = Collection->FindUVLayer(0);
		for (int32 Idx = 0; Idx < NumNewVertices; ++Idx)
		{
			Vertices[Idx] = FVector3f(RawVertexArray[3 * Idx], RawVertexArray[3 * Idx + 1], RawVertexArray[3 * Idx + 2]);
			(*UV0)[Idx] = FVector2f::ZeroVector;

			Colors[Idx] = Collection->Defaults.DefaultVertexColor;
			BoneMap[Idx] = 0;
		}

		Transform[0] = FTransform3f(FVector3f(0.f,0.f,0.f));

		// set the index information
		TArray<FVector3f> FaceNormals;
		FaceNormals.SetNum(NumNewIndices);
		for (int32 Idx = 0; Idx < NumNewIndices; ++Idx)
		{
			int32 VertexIdx1, VertexIdx2, VertexIdx3;
			if (!ReverseVertexOrder)
			{
				VertexIdx1 = RawIndicesArray[3 * Idx];
				VertexIdx2 = RawIndicesArray[3 * Idx + 1];
				VertexIdx3 = RawIndicesArray[3 * Idx + 2];
			}
			else
			{
				VertexIdx1 = RawIndicesArray[3 * Idx];
				VertexIdx2 = RawIndicesArray[3 * Idx + 2];
				VertexIdx3 = RawIndicesArray[3 * Idx + 1];
			}

			Indices[Idx] = FIntVector(VertexIdx1, VertexIdx2, VertexIdx3);
			Visible[Idx] = true;
			Internal[Idx] = false;
			MaterialID[Idx] = 0;
			MaterialIndex[Idx] = Idx;

			const FVector3f Edge1 = Vertices[VertexIdx1] - Vertices[VertexIdx2];
			const FVector3f Edge2 = Vertices[VertexIdx1] - Vertices[VertexIdx3];
			FaceNormals[Idx] = (Edge2 ^ Edge1).GetSafeNormal();
		}

		// Compute vertexNormals
		TArray<FVector3f> VertexNormals;
		VertexNormals.SetNum(NumNewVertices);
		for (int32 Idx = 0; Idx < NumNewVertices; ++Idx)
		{
			VertexNormals[Idx] = FVector3f(0.f, 0.f, 0.f);
		}

		for (int32 Idx = 0; Idx < NumNewIndices; ++Idx)
		{
			VertexNormals[Indices[Idx][0]] += FaceNormals[Idx];
			VertexNormals[Indices[Idx][1]] += FaceNormals[Idx];
			VertexNormals[Indices[Idx][2]] += FaceNormals[Idx];
		}

		for (int32 Idx = 0; Idx < NumNewVertices; ++Idx)
		{
			Normals[Idx] = (VertexNormals[Idx] / 3.f).GetSafeNormal();
		}

		for (int IndexIdx = 0; IndexIdx < NumNewIndices; IndexIdx++)
		{
			FIntVector Tri = Indices[IndexIdx];
			for (int idx = 0; idx < 3; idx++)
			{
				const FVector3f Normal = Normals[Tri[idx]];
				const FVector3f Edge = (Vertices[Tri[(idx + 1) % 3]] - Vertices[Tri[idx]]);
				TangentU[Tri[idx]] = (Edge ^ Normal).GetSafeNormal();
				TangentV[Tri[idx]] = (Normal ^ TangentU[Tri[idx]]).GetSafeNormal();
			}
		}

		// Build the Geometry Group
		GeometryCollection::AddGeometryProperties(Collection);

		// add a material section
		TManagedArray<FGeometryCollectionSection>& Sections = Collection->Sections;
		int Element = Collection->AddElements(1, FGeometryCollection::MaterialGroup);
		Sections[Element].MaterialID = 0;
		Sections[Element].FirstIndex = 0;
		Sections[Element].NumTriangles = Indices.Num();
		Sections[Element].MinVertexIndex = 0;
		Sections[Element].MaxVertexIndex = Vertices.Num() - 1;
	}
}

void FGeometryCollection::WriteDataToHeaderFile(const FString &Name, const FString &Path)
{
	using namespace std;

	static const FString DataFilePath = "D:";
	FString FullPath = (Path.IsEmpty() || Path.Equals(TEXT("None"))) ? DataFilePath : Path;
	FullPath.RemoveFromEnd("\\");
	FullPath += "\\" + Name + ".h";

	ofstream DataFile;
	DataFile.open(string(TCHAR_TO_UTF8(*FullPath)));
	DataFile << "// Copyright Epic Games, Inc. All Rights Reserved." << endl << endl;
	DataFile << "#pragma once" << endl << endl;
	DataFile << "class " << TCHAR_TO_UTF8(*Name) << endl;
	DataFile << "{" << endl;
	DataFile << "public:" << endl;
	DataFile << "    " << TCHAR_TO_UTF8(*Name) << "();" << endl;
	DataFile << "    ~" << TCHAR_TO_UTF8(*Name) << "() {};" << endl << endl;
	DataFile << "    static const TArray<float>	RawVertexArray;" << endl;
	DataFile << "    static const TArray<int32>	RawIndicesArray;" << endl;
	DataFile << "    static const TArray<int32>	RawBoneMapArray;" << endl;
	DataFile << "    static const TArray<FTransform> RawTransformArray;" << endl;
	DataFile << "    static const TArray<int32> RawParentArray;" << endl;
	DataFile << "    static const TArray<TSet<int32>> RawChildrenArray;" << endl;
	DataFile << "    static const TArray<int32> RawSimulationTypeArray;" << endl;
	DataFile << "    static const TArray<int32> RawStatusFlagsArray;" << endl;
	DataFile << "};" << endl << endl;
	DataFile << "const TArray<float> " << TCHAR_TO_UTF8(*Name) << "::RawVertexArray = {" << endl;

	int32 NumVertices = NumElements(FGeometryCollection::VerticesGroup);
	const TManagedArray<FVector3f>& VertexArray = Vertex;
	for (int32 IdxVertex = 0; IdxVertex < NumVertices; ++IdxVertex)
	{
		DataFile << "                                                    " <<
			VertexArray[IdxVertex].X << ", " <<
			VertexArray[IdxVertex].Y << ", " <<
			VertexArray[IdxVertex].Z << ", " << endl;
	}
	DataFile << "};" << endl << endl;
	DataFile << "const TArray<int32> " << TCHAR_TO_UTF8(*Name) << "::RawIndicesArray = {" << endl;

	int32 NumFaces = NumElements(FGeometryCollection::FacesGroup);
	for (int32 IdxFace = 0; IdxFace < NumFaces; ++IdxFace)
	{
		DataFile << "                                                    " <<
			Indices[IdxFace].X << ", " <<
			Indices[IdxFace].Y << ", " <<
			Indices[IdxFace].Z << ", " << endl;
	}

	DataFile << "};" << endl << endl;
	DataFile << "const TArray<int32> " << TCHAR_TO_UTF8(*Name) << "::RawBoneMapArray = {" << endl;

	for (int32 IdxVertex = 0; IdxVertex < NumVertices; ++IdxVertex)
	{
		DataFile << "                                                    " <<
			BoneMap[IdxVertex] << ", " << endl;
	}
	DataFile << "};" << endl << endl;

	DataFile << "const TArray<FTransform> " << TCHAR_TO_UTF8(*Name) << "::RawTransformArray = {" << endl;

	int32 NumTransforms = NumElements(FGeometryCollection::TransformGroup);
	const TManagedArray<FTransform3f>& TransformArray = Transform;
	for (int32 IdxTransform = 0; IdxTransform < NumTransforms; ++IdxTransform)
	{
		FQuat4f Rotation = TransformArray[IdxTransform].GetRotation();
		FVector3f Translation = TransformArray[IdxTransform].GetTranslation();
		FVector3f Scale3D = TransformArray[IdxTransform].GetScale3D();

		DataFile << "   FTransform(FQuat(" <<
			Rotation.X << ", " <<
			Rotation.Y << ", " <<
			Rotation.Z << ", " <<
			Rotation.W << "), " <<
			"FVector(" <<
			Translation.X << ", " <<
			Translation.Y << ", " <<
			Translation.Z << "), " <<
			"FVector(" <<
			Scale3D.X << ", " <<
			Scale3D.Y << ", " <<
			Scale3D.Z << ")), " << endl;
	}
	DataFile << "};" << endl << endl;

	// Write BoneHierarchy array
	DataFile << "const TArray<FGeometryCollectionBoneNode> " << TCHAR_TO_UTF8(*Name) << "::RawBoneHierarchyArray = {" << endl;

	for (int32 IdxTransform = 0; IdxTransform < NumTransforms; ++IdxTransform)
	{
		DataFile << "   FGeometryCollectionBoneNode(" <<
			Parent[IdxTransform] << ", " <<
			SimulationType[IdxTransform] << "), " <<
			StatusFlags[IdxTransform] << "), " << endl;
	}

	DataFile << "};" << endl << endl;
	DataFile.close();
}

void FGeometryCollection::WriteDataToOBJFile(const FString &Name, const FString &Path, const bool WriteTopology, const bool WriteAuxStructures)
{
	using namespace std;

	static const FString DataFilePath = "D:";

	int32 NumVertices = NumElements(FGeometryCollection::VerticesGroup);
	int32 NumFaces = NumElements(FGeometryCollection::FacesGroup);

	TArray<FTransform> GlobalTransformArray;
	GeometryCollectionAlgo::GlobalMatrices(Transform, Parent, GlobalTransformArray);

	TArray<FVector3f> VertexInWorldArray;
	VertexInWorldArray.SetNum(NumVertices);

	for (int32 IdxVertex = 0; IdxVertex < NumVertices; ++IdxVertex)
	{
		FTransform LocalTransform = GlobalTransformArray[BoneMap[IdxVertex]];
		FVector3f VertexInWorld = (FVector3f)LocalTransform.TransformPosition((FVector)Vertex[IdxVertex]);

		VertexInWorldArray[IdxVertex] = VertexInWorld;
	}

	ofstream DataFile;
	if (WriteTopology)
	{
		FString FullPath = (Path.IsEmpty() || Path.Equals(TEXT("None"))) ? DataFilePath : Path;
		FullPath.RemoveFromEnd("\\");
		FullPath += "\\" + Name + ".obj";

		DataFile.open(string(TCHAR_TO_UTF8(*FullPath)));

		DataFile << "# File exported from Unreal Engine" << endl;
		DataFile << "# " << NumVertices << " points" << endl;
		DataFile << "# " << NumVertices * 3 << " vertices" << endl;
		DataFile << "# " << NumFaces << " primitives" << endl;
		DataFile << "g" << endl;
		for (int32 IdxVertex = 0; IdxVertex < NumVertices; ++IdxVertex)
		{
			DataFile << "v " << VertexInWorldArray[IdxVertex].X << " " <<
				VertexInWorldArray[IdxVertex].Y << " " <<
				VertexInWorldArray[IdxVertex].Z << endl;
		}
		DataFile << "g" << endl;

		// FaceIndex in the OBJ format starts with 1
		for (int32 IdxFace = 0; IdxFace < NumFaces; ++IdxFace)
		{
			DataFile << "f " << Indices[IdxFace].X + 1 << " " <<
				Indices[IdxFace].Z + 1 << " " <<
				Indices[IdxFace].Y + 1 << endl;
		}
		DataFile << endl;
		DataFile.close();
	}
	if(WriteAuxStructures && HasAttribute("VertexVisibility", FGeometryCollection::VerticesGroup))
	{
		FString FullPath = (Path.IsEmpty() || Path.Equals(TEXT("None"))) ? DataFilePath : Path;
		FullPath.RemoveFromEnd("\\");
		FullPath += "\\" + Name + "_VertexVisibility.obj";

		DataFile.open(string(TCHAR_TO_UTF8(*FullPath)));
		DataFile << "# Vertex Visibility - vertices whose visibility flag are true" << endl;

		const TManagedArray<bool>& VertexVisibility = ModifyAttribute<bool>("VertexVisibility", FGeometryCollection::VerticesGroup);
		int num = 0;
		for (int32 IdxVertex = 0; IdxVertex < NumVertices; ++IdxVertex)
		{
			if (VertexVisibility[IdxVertex])
			{
				num++;
			}
		}
		DataFile << "# " << num << " Vertices" << endl;

		DataFile << "g" << endl;
		for (int32 IdxVertex = 0; IdxVertex < NumVertices; ++IdxVertex)
		{
			if(VertexVisibility[IdxVertex])
			{
				DataFile << "v " 
					<< VertexInWorldArray[IdxVertex].X << " " 
					<< VertexInWorldArray[IdxVertex].Y << " " 
					<< VertexInWorldArray[IdxVertex].Z << endl;
			}
		}
		DataFile << endl;
		DataFile.close();
	}
}

FGeometryCollection* FGeometryCollection::NewGeometryCollection(const TArray<float>& RawVertexArray,
																const TArray<int32>& RawIndicesArray,
																const TArray<int32>& RawBoneMapArray,
																const TArray<FTransform>& RawTransformArray,
																const TManagedArray<int32>& RawLevelArray,
																const TManagedArray<int32>& RawParentArray,
																const TManagedArray<TSet<int32>>& RawChildrenArray,
																const TManagedArray<int32>& RawSimulationTypeArray,
															    const TManagedArray<int32>& RawStatusFlagsArray,
																const FGeometryCollectionDefaults RawDefaults)
{
	FGeometryCollection* RestCollection = new FGeometryCollection(RawDefaults);

	int NumNewVertices = RawVertexArray.Num() / 3;
	int VerticesIndex = RestCollection->AddElements(NumNewVertices, FGeometryCollection::VerticesGroup);

	int NumNewIndices = RawIndicesArray.Num() / 3;
	int IndicesIndex = RestCollection->AddElements(NumNewIndices, FGeometryCollection::FacesGroup);

	TManagedArray<FVector3f>& Vertices = RestCollection->Vertex;
	TManagedArray<FVector3f>&  Normals = RestCollection->Normal;
	TManagedArray<FVector3f>&  TangentU = RestCollection->TangentU;
	TManagedArray<FVector3f>&  TangentV = RestCollection->TangentV;
	TManagedArray<FLinearColor>&  Colors = RestCollection->Color;
	TManagedArray<int32>& BoneMap = RestCollection->BoneMap;
	TManagedArray<FIntVector>&  Indices = RestCollection->Indices;
	TManagedArray<bool>&  Visible = RestCollection->Visible;
	TManagedArray<int32>&  MaterialID = RestCollection->MaterialID;
	TManagedArray<int32>&  MaterialIndex = RestCollection->MaterialIndex;
	TManagedArray<bool>& Internal = RestCollection->Internal;
	TManagedArray<FTransform3f>&  Transform = RestCollection->Transform;
	TManagedArray<int32>& Parent = RestCollection->Parent;
	TManagedArray<TSet<int32>>& Children = RestCollection->Children;
	TManagedArray<int32>& SimulationType = RestCollection->SimulationType;
	TManagedArray<int32>& StatusFlags = RestCollection->StatusFlags;
	TManagedArray<int32>& InitialDynamicState = RestCollection->InitialDynamicState;

	RestCollection->SetNumUVLayers(1);

	// set the vertex information
	TManagedArray<FVector2f>* UV0 = RestCollection->FindUVLayer(0);
	for (int32 Idx = 0; Idx < NumNewVertices; ++Idx)
	{
		Vertices[Idx] = FVector3f(RawVertexArray[3 * Idx], RawVertexArray[3 * Idx + 1], RawVertexArray[3 * Idx + 2]);
		BoneMap[Idx] = RawBoneMapArray[Idx];
		(*UV0)[Idx] = FVector2f::ZeroVector;

		Colors[Idx] = RestCollection->Defaults.DefaultVertexColor;
	}

	// Transforms
	int NumNewTransforms = RawTransformArray.Num(); // 1 particle for this geometry structure
	int TransformIndex = RestCollection->AddElements(NumNewTransforms, FGeometryCollection::TransformGroup);

	for (int32 Idx = 0; Idx < NumNewTransforms; ++Idx)
	{
		Transform[Idx] = FTransform3f(RawTransformArray[Idx]);
		Transform[Idx].NormalizeRotation();

		Parent[Idx] = RawParentArray[Idx];
		if (RawChildrenArray.Num() > 0)
		{
			Children[Idx] = RawChildrenArray[Idx];
		}
		SimulationType[Idx] = RawSimulationTypeArray[Idx];
		StatusFlags[Idx] = RawStatusFlagsArray[Idx];

		for (int32 Idx1 = 0; Idx1 < NumNewTransforms; ++Idx1)
		{
			if (RawParentArray[Idx1] == Idx)
			{
				Children[Idx].Add(Idx1);
			}
		}

	}

	// set the index information
	TArray<FVector3f> FaceNormals;
	FaceNormals.SetNum(NumNewIndices);
	for (int32 Idx = 0; Idx < NumNewIndices; ++Idx)
	{
		int32 VertexIdx1, VertexIdx2, VertexIdx3;
		VertexIdx1 = RawIndicesArray[3 * Idx];
		VertexIdx2 = RawIndicesArray[3 * Idx + 1];
		VertexIdx3 = RawIndicesArray[3 * Idx + 2];

		Indices[Idx] = FIntVector(VertexIdx1, VertexIdx2, VertexIdx3);
		Visible[Idx] = true;
		Internal[Idx] = false;
		MaterialID[Idx] = 0;
		MaterialIndex[Idx] = Idx;

		const FVector3f Edge1 = Vertices[VertexIdx1] - Vertices[VertexIdx2];
		const FVector3f Edge2 = Vertices[VertexIdx1] - Vertices[VertexIdx3];
		FaceNormals[Idx] = (Edge2 ^ Edge1).GetSafeNormal();
	}

	// Compute vertexNormals
	TArray<FVector3f> VertexNormals;
	VertexNormals.SetNum(NumNewVertices);
	for (int32 Idx = 0; Idx < NumNewVertices; ++Idx)
	{
		VertexNormals[Idx] = FVector3f(0.f, 0.f, 0.f);
	}

	for (int32 Idx = 0; Idx < NumNewIndices; ++Idx)
	{
		VertexNormals[Indices[Idx][0]] += FaceNormals[Idx];
		VertexNormals[Indices[Idx][1]] += FaceNormals[Idx];
		VertexNormals[Indices[Idx][2]] += FaceNormals[Idx];
	}

	for (int32 Idx = 0; Idx < NumNewVertices; ++Idx)
	{
		Normals[Idx] = (VertexNormals[Idx] / 3.f).GetSafeNormal();
	}

	for (int IndexIdx = 0; IndexIdx < NumNewIndices; IndexIdx++)
	{
		FIntVector Tri = Indices[IndexIdx];
		for (int idx = 0; idx < 3; idx++)
		{
			const FVector3f Normal = Normals[Tri[idx]];
			const FVector3f Edge = (Vertices[Tri[(idx + 1) % 3]] - Vertices[Tri[idx]]);
			TangentU[Tri[idx]] = (Edge ^ Normal).GetSafeNormal();
			TangentV[Tri[idx]] = (Normal ^ TangentU[Tri[idx]]).GetSafeNormal();
		}
	}

	// Build the Geometry Group
	GeometryCollection::AddGeometryProperties(RestCollection);

	FGeometryCollectionProximityUtility ProximityUtility(RestCollection);
	ProximityUtility.UpdateProximity();

	// add a material section
	TManagedArray<FGeometryCollectionSection>&  Sections = RestCollection->Sections;
	int Element = RestCollection->AddElements(1, FGeometryCollection::MaterialGroup);
	Sections[Element].MaterialID = 0;
	Sections[Element].FirstIndex = 0;
	Sections[Element].NumTriangles = Indices.Num();
	Sections[Element].MinVertexIndex = 0;
	Sections[Element].MaxVertexIndex = Vertices.Num() - 1;

	return RestCollection;
}


TArray<TArray<int32>> FGeometryCollection::ConnectionGraph()
{
	int32 NumTransforms = NumElements(TransformGroup);

	TArray<TArray<int32>> Connectivity;
	Connectivity.Init(TArray<int32>(), NumTransforms);

	TArray<FTransform3f> GlobalMatrices;
	GeometryCollectionAlgo::GlobalMatrices(Transform, Parent, GlobalMatrices);

	TArray<FVector> Pts;
	TMap< int32,int32> Remap;
	for (int32 TransformGroupIndex = 0; TransformGroupIndex < NumTransforms; ++TransformGroupIndex)
	{
		if (IsGeometry(TransformGroupIndex))
		{
			Remap.Add(Pts.Num(), TransformGroupIndex);
			Pts.Add(FVector(GlobalMatrices[TransformGroupIndex].GetTranslation()));
		}
	}

	TArray<TArray<int>> Neighbors;
	VoronoiNeighbors(Pts, Neighbors);

	for (int i = 0; i < Neighbors.Num(); i++)
	{
		for (int j = 0; j < Neighbors[i].Num(); j++)
		{
			Connectivity[Remap[i]].Add(Remap[Neighbors[i][j]]);
		}
	}
	return Connectivity;
}

void FGeometryCollection::UpdateOldAttributeNames()
{

	// Faces Group
	int32 NumOldGeometryElements = this->NumElements("Geometry");
	check(!this->NumElements(FGeometryCollection::FacesGroup));
	this->AddElements(NumOldGeometryElements, FGeometryCollection::FacesGroup);

	TArray<FName> GeometryAttributes = this->AttributeNames("Geometry");

	const TManagedArray<FIntVector> & OldIndices = this->GetAttribute<FIntVector>("Indices", "Geometry");
	const TManagedArray<bool> & OldVisible = this->GetAttribute<bool>("Visible", "Geometry");
	const TManagedArray<int32> & OldMaterialIndex = this->GetAttribute<int32>("MaterialIndex", "Geometry");
	const TManagedArray<int32> & OldMaterialID = this->GetAttribute<int32>("MaterialID", "Geometry");
	for (int i = NumOldGeometryElements - 1; 0 <= i; i--)
	{
		this->Indices[i] = OldIndices[i];
		this->Visible[i] = OldVisible[i];
		this->MaterialIndex[i] = OldMaterialIndex[i];
		this->MaterialID[i] = OldMaterialID[i];
	}
	this->RemoveAttribute("Indices", "Geometry");
	this->RemoveAttribute("Visible", "Geometry");
	this->RemoveAttribute("MaterialIndex", "Geometry");
	this->RemoveAttribute("MaterialID", "Geometry");

	// reset the geometry group
	TArray<int32> DelArray;
	GeometryCollectionAlgo::ContiguousArray(DelArray, NumOldGeometryElements);
	FManagedArrayCollection::FProcessingParameters Params;
	Params.bDoValidation = false;
	Params.bReindexDependentAttibutes = false;
	Super::RemoveElements("Geometry", DelArray, Params);


	// Geometry Group
	TArray<FName> StructureAttributes = this->AttributeNames("Structure");

	int32 NumOldStructureElements = this->NumElements("Structure");
	check(!this->NumElements(FGeometryCollection::GeometryGroup));
	this->AddElements(NumOldStructureElements, FGeometryCollection::GeometryGroup);

	const TManagedArray<int32> & OldTransformIndex = this->GetAttribute<int32>("TransformIndex", "Structure");
	const TManagedArray<FBox> & OldBoundingBox = this->GetAttribute<FBox>("BoundingBox", "Structure");
	const TManagedArray<float> & OldInnerRadius = this->GetAttribute<float>("InnerRadius", "Structure");
	const TManagedArray<float> & OldOuterRadius = this->GetAttribute<float>("OuterRadius", "Structure");
	const TManagedArray<int32> & OldVertexStart = this->GetAttribute<int32>("VertexStart", "Structure");
	const TManagedArray<int32> & OldVertexCount = this->GetAttribute<int32>("VertexCount", "Structure");
	const TManagedArray<int32> & OldFaceStart = this->GetAttribute<int32>("FaceStart", "Structure");
	const TManagedArray<int32> & OldFaceCount = this->GetAttribute<int32>("FaceCount", "Structure");
	for (int i = NumOldStructureElements - 1; 0 <= i; i--)
	{
		this->TransformIndex[i] = OldTransformIndex[i];
		this->BoundingBox[i] = OldBoundingBox[i];
		this->InnerRadius[i] = OldInnerRadius[i];
		this->OuterRadius[i] = OldOuterRadius[i];
		this->VertexStart[i] = OldVertexStart[i];
		this->VertexCount[i] = OldVertexCount[i];
		this->FaceStart[i] = OldFaceStart[i];
		this->FaceCount[i] = OldFaceCount[i];
	}
	this->RemoveGroup("Structure");
}

TArray<int32> FGeometryCollection::TransformSelectionToGeometryIndices(const TArray<int32>& Transforms)
{
	TArray<int32> Geometries;
	for (const int32& TransformIdx : Transforms)
	{
		if (TransformToGeometryIndex.IsValidIndex(TransformIdx)	&& IsGeometry(TransformIdx))
		{
			Geometries.Add(TransformToGeometryIndex[TransformIdx]);
		}
	}
	return Geometries;
}

