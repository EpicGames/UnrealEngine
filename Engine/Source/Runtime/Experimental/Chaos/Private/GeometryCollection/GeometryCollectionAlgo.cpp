// Copyright Epic Games, Inc. All Rights Reserved.
/*=============================================================================
	GeometryCollection.cpp: FGeometryCollection methods.
=============================================================================*/

#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/RecordedTransformTrack.h"
#include "GeometryCollection/Facades/CollectionHierarchyFacade.h"
#include "GeometryCollection/Facades/CollectionTransformFacade.h"
#include "GeometryCollectionProxyData.h"
#include "Async/ParallelFor.h"

DEFINE_LOG_CATEGORY_STATIC(GeometryCollectionAlgoLog, Log, All);

namespace GeometryCollectionAlgo
{

	template<typename TransformType>
	void PrintParentHierarchyRecursive(int32 Index
		, const TManagedArray<TransformType>& Transform
		, const TManagedArray<int32>& Parent
		, const TManagedArray<TSet<int32>>& Children
		, const TManagedArray<int32>& SimulationType
		, const TManagedArray<FString>& BoneName
		, int8 Tab = 0
	)
	{
		check(Index >= 0);
		check(Index < Transform.Num());
		FString Buffer;
		Buffer += FString::Printf(TEXT("R(%+6.2f,%+6.2f,%+6.2f,%+6.2f) "), Transform[Index].GetRotation().X, Transform[Index].GetRotation().Y, Transform[Index].GetRotation().Z, Transform[Index].GetRotation().W);
		Buffer += FString::Printf(TEXT("S(%+6.2f,%+6.2f,%+6.2f)"), Transform[Index].GetScale3D().X, Transform[Index].GetScale3D().Y, Transform[Index].GetScale3D().Z);
		Buffer += FString::Printf(TEXT("T(%+6.2f,%+6.2f,%+6.2f)"), Transform[Index].GetTranslation().X, Transform[Index].GetTranslation().Y, Transform[Index].GetTranslation().Z);
		for (int Tdx = 0; Tdx < Tab; Tdx++)
			Buffer += " ";
		Buffer += FString::Printf(TEXT("[%d] Name : '%s'  Parent %d  SimulationType %d"), Index, *BoneName[Index], Parent[Index], SimulationType[Index]);

		UE_LOGF(GeometryCollectionAlgoLog, Log, "%ls", *Buffer);

		for (auto& ChildIndex : Children[Index])
		{
			PrintParentHierarchyRecursive(ChildIndex, Transform, Parent, Children, SimulationType, BoneName, Tab + 3);
		}
	}


	void PrintParentHierarchy(const FGeometryCollection* Collection)
	{
		check(Collection);

		const TManagedArray<FTransform3f>& Transform = Collection->Transform;
		const TManagedArray<FString>& BoneNames = Collection->BoneName;
		const TManagedArray<int32>& Parent = Collection->Parent;
		const TManagedArray<TSet<int32>>& Children = Collection->Children;
		const TManagedArray<int32>& SimulationType = Collection->SimulationType;
		int32 NumParticles = Collection->NumElements(FGeometryCollection::TransformGroup);
		for (int32 Index = 0; Index < NumParticles; Index++)
		{
			if (Parent[Index] == FGeometryCollection::Invalid)
			{
				PrintParentHierarchyRecursive(Index, Transform, Parent, Children, SimulationType, BoneNames);
			}
		}
	}

	void ContiguousArray(TArray<int32> & Array, int32 Length)
	{
		Array.SetNumUninitialized(Length);
		for (int i = 0; i < Length; i++)
		{
			Array[i] = i;
		}
	}

	void BuildIncrementMask(const TArray<int32> & SortedDeletionList, const int32 & Size, TArray<int32> & Mask)
	{
		Mask.SetNumUninitialized(Size);
		for (int Index = 0, DelIndex = 0; Index < Size; Index++)
		{

			Mask[Index] = DelIndex;

			if (DelIndex < SortedDeletionList.Num() && Index == SortedDeletionList[DelIndex])
			{
				DelIndex++;
			}

		}
	}

	void BuildLookupMask(const TArray<int32> & SortedDeletionList, const int32 & Size, TArray<bool> & Mask)
	{
		Mask.Init(false, Size);
		for (int Index = 0; Index < SortedDeletionList.Num(); Index++)
		{
			if (SortedDeletionList[Index] < Size)
				Mask[SortedDeletionList[Index]] = true;
			else
				break;
		}
	}


	void BuildTransformGroupToGeometryGroupMap(const FGeometryCollection& GeometryCollection, TArray<int32> & TransformToGeometry)
	{
		int32 NumGeometryGroup = GeometryCollection.NumElements(FGeometryCollection::GeometryGroup);
		const TManagedArray<int32>& TransformIndex = GeometryCollection.TransformIndex;
		TransformToGeometry.Init(FGeometryCollection::Invalid, GeometryCollection.NumElements(FGeometryCollection::TransformGroup));
		for (int32 i = 0; i < NumGeometryGroup; i++)
		{
			check(TransformIndex[i] != FGeometryCollection::Invalid);
			TransformToGeometry[TransformIndex[i]] = i;
		}
	}


	void BuildFaceGroupToGeometryGroupMap(const FGeometryCollection& GeometryCollection, const TArray<int32>& TransformToGeometryMap, TArray<int32> & FaceToGeometry)
	{
		check(TransformToGeometryMap.Num() == GeometryCollection.NumElements(FGeometryCollection::TransformGroup));
		const TManagedArray<FIntVector>& Indices = GeometryCollection.Indices;
		const TManagedArray<int32>& BoneMap = GeometryCollection.BoneMap;

		int32 NumTransforms = TransformToGeometryMap.Num();
		int32 NumFaces = GeometryCollection.NumElements(FGeometryCollection::FacesGroup);
		FaceToGeometry.Init(FGeometryCollection::Invalid, NumFaces);
		for (int32 i = 0; i < NumFaces; i++)
		{
			check(0 <= Indices[i][0] && Indices[i][0] < TransformToGeometryMap.Num());
			FaceToGeometry[i] = TransformToGeometryMap[Indices[i][0]];
		}
	}


	void ValidateSortedList(const TArray<int32>&SortedDeletionList, const int32 & ListSize)
	{
		int32 PreviousValue = -1;
		int32 DeletionListSize = SortedDeletionList.Num();
		if (DeletionListSize)
		{
			ensureMsgf(DeletionListSize <= ListSize, TEXT("TManagedArray::ValidateSortedList( DeletionList ) DeletionList larger than array"));
			for (int32 Index = 0; Index < SortedDeletionList.Num(); Index++)
			{
				ensureMsgf(PreviousValue < SortedDeletionList[Index], TEXT("TManagedArray::ValidateSortedList( DeletionList ) DeletionList not sorted"));
				ensureMsgf(0 <= SortedDeletionList[Index] && SortedDeletionList[Index] < ListSize, TEXT("TManagedArray::ValidateSortedList( DeletionList ) Index out of range"));
				PreviousValue = SortedDeletionList[Index];
			}
		}
	}


	FVector AveragePosition(FGeometryCollection* Collection, const TArray<int32>& Indices)
	{
		TManagedArray<FTransform3f>& Transform = Collection->Transform;
		int32 NumIndices = Indices.Num();

		FVector3f Translation(0);
		for (int32 Index = 0; Index < NumIndices; Index++)
		{
			Translation += Transform[Indices[Index]].GetTranslation();
		}
		if (NumIndices > 1)
		{
			Translation /= static_cast<float>(NumIndices);
		}
		return FVector(Translation);
	}

	bool HasMultipleRoots(FGeometryCollection * Collection)
	{
		int32 ParentCount = 0;
		TManagedArray<int32>& Parents = Collection->Parent;
		for (int32 i = 0; i < Parents.Num(); i++)
		{
			if (Parents[i] == FGeometryCollection::Invalid) ParentCount++;
			if (ParentCount > 1) return true;
		}
		return false;
	}

	bool HasCycle(const TManagedArray<int32>& Parents, int32 Node)
	{
		return GeometryCollection::Facades::FCollectionTransformFacade::HasCycle(Parents, Node);
	}
	bool HasCycle(const TManagedArray<int32>& Parents, const TArray<int32>& SelectedBones)
	{
		return GeometryCollection::Facades::FCollectionTransformFacade::HasCycle(Parents, SelectedBones);
	}

	void ParentTransform(FManagedArrayCollection* ManagedArrayCollection, const int32 TransformIndex, const int32 ChildIndex)
	{
		if (ManagedArrayCollection)
		{
			GeometryCollection::Facades::FCollectionTransformFacade Facade(*ManagedArrayCollection);
			Facade.ParentTransform(TransformIndex, ChildIndex);
		}
	}

	void ParentTransforms(FManagedArrayCollection* ManagedArrayCollection, const int32 TransformIndex, const TArray<int32>& SelectedBones)
	{
		if (ManagedArrayCollection)
		{
			GeometryCollection::Facades::FCollectionTransformFacade Facade(*ManagedArrayCollection);
			Facade.ParentTransforms(TransformIndex, SelectedBones);
		}
	}

	void UnparentTransform(FManagedArrayCollection* ManagedArrayCollection, const int32 ChildIndex)
	{
		if (ManagedArrayCollection)
		{
			GeometryCollection::Facades::FCollectionTransformFacade Facade(*ManagedArrayCollection);
			Facade.UnparentTransform(ChildIndex);
		}
	}

	// Helper type for computing global matrices
	using FIndicesNeedMatricesArray = TArray<int32, TInlineAllocator<16>>;

	// return false if we failed to get the indices, which can happen if the parent array does not describe a valid tree (e.g., if it contains a loop)
	bool GlobalMatricesGetIndicesToProcessHelper(const int32 Index, const FGeometryDynamicCollection& DynamicCollection, int32 ParentNum, TArray<bool>& IsTransformComputed, FIndicesNeedMatricesArray& OutToProcess)
	{
		checkSlow(Index != FGeometryCollection::Invalid);
		checkSlow(OutToProcess.IsEmpty());

		OutToProcess.Add(Index);
		bool bFoundRootOrComputed = false;
		const int32 MaxDepth = ParentNum;
		while (OutToProcess.Num() <= MaxDepth)
		{
			const int32 Parent = DynamicCollection.GetParent(OutToProcess.Last());
			if (Parent == INDEX_NONE || IsTransformComputed[Parent])
			{
				bFoundRootOrComputed = true;
				break;
			}
			OutToProcess.Add(Parent);
		}
		if (!ensureMsgf(bFoundRootOrComputed, TEXT("Geometry Collection has invalid parent hierarchy, could not find root to create global transforms")))
		{
			return false;
		}

		return true;
	}

	
	// return false if we failed to get the indices, which can happen if the parent array does not describe a valid tree (e.g., if it contains a loop)
	bool GlobalMatricesGetIndicesToProcessHelper(const int32 Index, const TManagedArray<int32>& Parents, TArray<bool>& IsTransformComputed, FIndicesNeedMatricesArray& OutToProcess)
	{
		checkSlow(Index != FGeometryCollection::Invalid);
		checkSlow(OutToProcess.IsEmpty());

		OutToProcess.Add(Index);
		bool bFoundRootOrComputed = false;
		const int32 MaxDepth = Parents.Num();
		while (OutToProcess.Num() <= MaxDepth)
		{
			int32 Parent = Parents[OutToProcess.Last()];
			if (Parent == INDEX_NONE || IsTransformComputed[Parent])
			{
				bFoundRootOrComputed = true;
				break;
			}
			OutToProcess.Add(Parent);
		}
		if (!ensureMsgf(bFoundRootOrComputed, TEXT("Geometry Collection has invalid parent hierarchy, could not find root to create global transforms")))
		{
			return false;
		}

		return true;
	}

	template<typename TransformType>
	void GlobalMatricesHelper(const int32 Index, const FGeometryDynamicCollection& DynamicCollection, TArray<bool>& IsTransformComputed, const TManagedArray<FTransform>* UniformScale, TArray<TransformType>& OutGlobalTransforms)
	{
		if (IsTransformComputed[Index])
		{
			return;
		}

		FIndicesNeedMatricesArray ToProcess;
		if (!GlobalMatricesGetIndicesToProcessHelper(Index, DynamicCollection, DynamicCollection.GetNumTransforms(), IsTransformComputed, ToProcess))
		{
			return;
		}

		while (!ToProcess.IsEmpty())
		{
			const int32 ProcessIndex = ToProcess.Pop(EAllowShrinking::No);
			const int32 ParentIndex = DynamicCollection.GetParent(ProcessIndex);
			TransformType Result = TransformType(DynamicCollection.GetTransform(ProcessIndex));
			if (ParentIndex != FGeometryCollection::Invalid)
			{
				Result *= OutGlobalTransforms[ParentIndex];
			}

			if (UniformScale)
			{
				OutGlobalTransforms[ProcessIndex] = TransformType((*UniformScale)[ProcessIndex]) * Result;
			}
			else
			{
				OutGlobalTransforms[ProcessIndex] = Result;
			}

			IsTransformComputed[ProcessIndex] = true;
		}
	}

	// #note: this version outputs FTransforms to support functionality for getting global matrices for an array of indices.
	template<typename TransformType, typename TransformTypeOut>
	void GlobalMatricesHelper(const int32 Index, const TManagedArray<int32>& Parents, const TManagedArray<TransformType>& Transform, TArray<bool>& IsTransformComputed, const TManagedArray<FTransform>* UniformScale, TArray<TransformTypeOut>& OutGlobalTransforms)
	{
		if (IsTransformComputed[Index])
		{
			return;
		}

		FIndicesNeedMatricesArray ToProcess;
		if (!GlobalMatricesGetIndicesToProcessHelper(Index, Parents, IsTransformComputed, ToProcess))
		{
			return;
		}

		while (!ToProcess.IsEmpty())
		{
			const int32 ProcessIndex = ToProcess.Pop(EAllowShrinking::No);
			const int32 ParentIndex = Parents[ProcessIndex];
			TransformTypeOut Result = TransformTypeOut(Transform[ProcessIndex]);
			if (ParentIndex != FGeometryCollection::Invalid)
			{
				Result *= OutGlobalTransforms[ParentIndex];
			}

			if (UniformScale)
			{
				OutGlobalTransforms[ProcessIndex] = TransformTypeOut((*UniformScale)[ProcessIndex]) * Result;
			}
			else
			{
				OutGlobalTransforms[ProcessIndex] = TransformTypeOut(Result);
			}

			IsTransformComputed[ProcessIndex] = true;
		}
	}

	template<typename TransformType>
	void GlobalMatricesHelper(const int32 Index, const TManagedArray<int32>& Parents, const TManagedArray<TransformType>& Transform, TArray<bool>& IsTransformComputed, const TManagedArray<FTransform>* UniformScale, TArray<FMatrix>& OutGlobalTransforms)
	{
		if (IsTransformComputed[Index])
		{
			return;
		}

		FIndicesNeedMatricesArray ToProcess;
		if (!GlobalMatricesGetIndicesToProcessHelper(Index, Parents, IsTransformComputed, ToProcess))
		{
			return;
		}

		while (!ToProcess.IsEmpty())
		{
			const int32 ProcessIndex = ToProcess.Pop(EAllowShrinking::No);
			const int32 ParentIndex = Parents[ProcessIndex];
			FMatrix Result = FTransform(Transform[ProcessIndex]).ToMatrixWithScale();
			if (ParentIndex != FGeometryCollection::Invalid)
			{
				Result *= OutGlobalTransforms[ParentIndex];
			}

			if (UniformScale)
			{
				OutGlobalTransforms[ProcessIndex] = (*UniformScale)[ProcessIndex].ToMatrixWithScale() * Result;
			}
			else
			{
				OutGlobalTransforms[ProcessIndex] = Result;
			}

			IsTransformComputed[ProcessIndex] = true;
		}
	}

	FTransform GlobalMatricesHelperForIndicesDynCol(const int32 Index, const FGeometryDynamicCollection& DynamicCollection, TArray<bool>& IsTransformComputed, const TManagedArray<FTransform>* UniformScale, TArray<FTransform>& TransformCache)
	{
		if (IsTransformComputed[Index])
		{
			return TransformCache[Index];
		}

		FIndicesNeedMatricesArray ToProcess;
		if (!GlobalMatricesGetIndicesToProcessHelper(Index, DynamicCollection, DynamicCollection.GetNumTransforms(), IsTransformComputed, ToProcess))
		{
			return TransformCache[Index];
		}

		while (!ToProcess.IsEmpty())
		{
			const int32 ProcessIndex = ToProcess.Pop(EAllowShrinking::No);
			const int32 ParentIndex = DynamicCollection.GetParent(ProcessIndex);
			FTransform Result = FTransform(DynamicCollection.GetTransform(ProcessIndex));
			if (ParentIndex != FGeometryCollection::Invalid)
			{
				Result *= TransformCache[ParentIndex];
			}

			if (UniformScale)
			{
				TransformCache[ProcessIndex] = (*UniformScale)[ProcessIndex] * Result;
			}
			else
			{
				TransformCache[ProcessIndex] = Result;
			}

			IsTransformComputed[ProcessIndex] = true;
		}

		return TransformCache[Index];
	}

	template<typename TransnformType>
	TransnformType GlobalMatricesHelperForIndices(const int32 Index, const TManagedArray<int32>& Parents, const TManagedArray<TransnformType>& Transform, TArray<bool>& IsTransformComputed, const TManagedArray<TransnformType>* UniformScale, TArray<FTransform>& TransformCache)
	{
		if (IsTransformComputed[Index])
		{
			return TransnformType(TransformCache[Index]);
		}

		FIndicesNeedMatricesArray ToProcess;
		if (!GlobalMatricesGetIndicesToProcessHelper(Index, Parents, IsTransformComputed, ToProcess))
		{
			return TransnformType(TransformCache[Index]);
		}

		while (!ToProcess.IsEmpty())
		{
			const int32 ProcessIndex = ToProcess.Pop(EAllowShrinking::No);
			const int32 ParentIndex = Parents[ProcessIndex];
			TransnformType Result = Transform[ProcessIndex];
			if (ParentIndex != FGeometryCollection::Invalid)
			{
				Result *= TransnformType(TransformCache[ParentIndex]);
			}

			if (UniformScale)
			{
				TransformCache[ProcessIndex] = FTransform((*UniformScale)[ProcessIndex] * Result);
			}
			else
			{
				TransformCache[ProcessIndex] = FTransform(Result);
			}

			IsTransformComputed[ProcessIndex] = true;
		}

		return TransnformType(TransformCache[Index]);
	}

	namespace Private 
	{
		FTransform GlobalMatrix(const FGeometryDynamicCollection& DynamicCollection, int32 Index)
		{
			FTransform Transform = FTransform::Identity;

			check(Index < DynamicCollection.GetNumTransforms())
			while (Index != FGeometryCollection::Invalid)
			{
				Transform = Transform * FTransform(DynamicCollection.GetTransform(Index));
				Index = DynamicCollection.GetParent(Index);
			}
			return Transform;
		}
	}

	template<typename TransformType>
	TransformType GlobalMatrixTemplate(const TManagedArray<TransformType>& RelativeTransforms, const TManagedArray<int32>& Parents, int32 Index)
	{
		TransformType Transform = TransformType::Identity;

		if (RelativeTransforms.IsValidIndex(Index))
		{
			do
			{
				Transform = Transform * RelativeTransforms[Index];
				Index = Parents[Index];
			} while (Index != FGeometryCollection::Invalid);
		}
		return Transform;
	}

	template<typename TransformType>
	TransformType GlobalMatrixTemplate(TArrayView<const TransformType> RelativeTransforms, TArrayView<const int32> Parents, int32 Index)
	{
		TransformType Transform = TransformType::Identity;

		if (RelativeTransforms.IsValidIndex(Index))
		{
			do
			{
				Transform = Transform * RelativeTransforms[Index];
				Index = Parents[Index];
			} while (Index != FGeometryCollection::Invalid);
		}
		return Transform;
	}


	FTransform GlobalMatrix(const TManagedArray<FTransform>& RelativeTransforms, const TManagedArray<int32>& Parents, int32 Index)
	{
		return GlobalMatrixTemplate<FTransform>(RelativeTransforms, Parents, Index);
	}

	FTransform3f GlobalMatrix3f(const TManagedArray<FTransform3f>& RelativeTransforms, const TManagedArray<int32>& Parents, int32 Index)
	{
		return GlobalMatrixTemplate<FTransform3f>(RelativeTransforms, Parents, Index);
	}

	FTransform GlobalMatrix(const TManagedArray<FTransform3f>& RelativeTransforms, const TManagedArray<int32>& Parents, int32 Index)
	{
		return FTransform(GlobalMatrixTemplate<FTransform3f>(RelativeTransforms, Parents, Index));
	}

	FTransform GlobalMatrix(TArrayView<const FTransform> RelativeTransforms, TArrayView<const int32> Parents, int32 Index)
	{
		return GlobalMatrixTemplate<FTransform>(RelativeTransforms, Parents, Index);
	}
	FTransform GlobalMatrix(TArrayView<const FTransform3f> RelativeTransforms, TArrayView<const int32> Parents, int32 Index)
	{
		return FTransform(GlobalMatrixTemplate<FTransform3f>(RelativeTransforms, Parents, Index));
	}

	namespace Private
	{
		void GlobalMatrices(const FGeometryDynamicCollection& DynamicCollection, const TArray<int32>& Indices, TArray<FTransform>& OutGlobalTransforms)
		{
			TArray<bool> IsTransformComputed;
			const int32 NumTransform = DynamicCollection.GetNumTransforms();
			IsTransformComputed.AddDefaulted(NumTransform);

			TArray<FTransform> TransformCache;
			TransformCache.SetNumUninitialized(NumTransform, EAllowShrinking::No);

			OutGlobalTransforms.SetNumUninitialized(Indices.Num(), EAllowShrinking::No);
			for (int Idx = 0; Idx < Indices.Num(); Idx++)
			{
				OutGlobalTransforms[Idx] = GlobalMatricesHelperForIndicesDynCol(Indices[Idx], DynamicCollection, IsTransformComputed, nullptr, TransformCache);
			}
		}
	}

	template<class TransformType, class TransformTypeOut>
	void GlobalMatricesTemplate(const TManagedArray<TransformType>& RelativeTransforms, const TManagedArray<int32>& Parents, const TArray<int32>& Indices, TArray<TransformTypeOut>& OutGlobalTransforms)
	{
		TArray<bool> IsTransformComputed;
		IsTransformComputed.AddDefaulted(RelativeTransforms.Num());

		TArray<FTransform> TransformCache;
		TransformCache.SetNumUninitialized(RelativeTransforms.Num(), EAllowShrinking::No);

		OutGlobalTransforms.SetNumUninitialized(Indices.Num(), EAllowShrinking::No);
		for (int Idx = 0; Idx < Indices.Num(); Idx++)
		{
			OutGlobalTransforms[Idx] = GlobalMatricesHelperForIndices<TransformType>(Indices[Idx], Parents, RelativeTransforms, IsTransformComputed, nullptr, TransformCache);
		}
	}

	void GlobalMatrices(const TManagedArray<FTransform>& RelativeTransforms, const TManagedArray<int32>& Parents, const TArray<int32>& Indices, TArray<FTransform>& Transforms)
	{
		GlobalMatricesTemplate<FTransform, FTransform>(RelativeTransforms, Parents, Indices, Transforms);
	}

	void GlobalMatrices(const TManagedArray<FTransform3f>& RelativeTransforms, const TManagedArray<int32>& Parents, const TArray<int32>& Indices, TArray<FTransform3f>& Transforms)
	{
		GlobalMatricesTemplate<FTransform3f, FTransform3f>(RelativeTransforms, Parents, Indices, Transforms);
	}

	void GlobalMatricesFromRoot(const int32 ParentTransformIndex, const TManagedArray<FTransform>& RelativeTransforms, const TManagedArray<TSet<int32>>& Children, TArray<FMatrix>& Transforms)
	{
		if (Children[ParentTransformIndex].Num() > 0)
		{
			for (int32 ChildIndex : Children[ParentTransformIndex])
			{
				Transforms[ChildIndex] = RelativeTransforms[ChildIndex].ToMatrixWithScale() * Transforms[ParentTransformIndex];
				GlobalMatricesFromRoot(ChildIndex, RelativeTransforms, Children, Transforms);
				
			}
		}
	}

	template<typename MatrixType>
	void GlobalMatrices(const TManagedArray<FTransform>& RelativeTransforms, const TManagedArray<int32>& Parents, const TManagedArray<FTransform>& UniformScale, TArray<MatrixType>& OutGlobalTransforms)
	{
		int32 NumTransforms = RelativeTransforms.Num();

		TArray<bool> IsTransformComputed;
		IsTransformComputed.AddDefaulted(NumTransforms);

		OutGlobalTransforms.SetNumUninitialized(NumTransforms, EAllowShrinking::No);

		for (int BoneIdx = 0; BoneIdx < NumTransforms; ++BoneIdx)
		{
			GlobalMatricesHelper(BoneIdx, Parents, RelativeTransforms, IsTransformComputed, &UniformScale, OutGlobalTransforms);
		}
	}

	namespace Private
	{
		void GlobalMatrices(const FGeometryDynamicCollection& DynamicCollection, TArray<FTransform>& OutGlobalTransforms)
		{
			int32 NumTransforms = DynamicCollection.GetNumTransforms();

			TArray<bool> IsTransformComputed;
			IsTransformComputed.AddDefaulted(NumTransforms);

			OutGlobalTransforms.SetNumUninitialized(NumTransforms, EAllowShrinking::No);

			for (int BoneIdx = 0; BoneIdx < NumTransforms; ++BoneIdx)
			{
				GlobalMatricesHelper(BoneIdx, DynamicCollection, IsTransformComputed, nullptr, OutGlobalTransforms);
			}
		}

		void GlobalMatrices(const FGeometryDynamicCollection& DynamicCollection, TArray<FTransform3f>& OutGlobalTransforms)
		{
			int32 NumTransforms = DynamicCollection.GetNumTransforms();

			TArray<bool> IsTransformComputed;
			IsTransformComputed.AddDefaulted(NumTransforms);

			OutGlobalTransforms.SetNumUninitialized(NumTransforms, EAllowShrinking::No);

			for (int BoneIdx = 0; BoneIdx < NumTransforms; ++BoneIdx)
			{
				GlobalMatricesHelper(BoneIdx, DynamicCollection, IsTransformComputed, nullptr, OutGlobalTransforms);
			}
		}
	}

	template<typename MatrixType, typename TransformType>
	void GlobalMatrices(const TManagedArray<TransformType>& RelativeTransforms, const TManagedArray<int32>& Parents, TArray<MatrixType>& OutGlobalTransforms)
	{
		int32 NumTransforms = RelativeTransforms.Num();

		TArray<bool> IsTransformComputed;
		IsTransformComputed.AddDefaulted(NumTransforms);

		OutGlobalTransforms.SetNumUninitialized(NumTransforms, EAllowShrinking::No);

		for (int BoneIdx = 0; BoneIdx < NumTransforms; ++BoneIdx)
		{
			GlobalMatricesHelper(BoneIdx, Parents, RelativeTransforms, IsTransformComputed, nullptr, OutGlobalTransforms);
		}
	}

	template void CHAOS_API GlobalMatrices<FTransform>(const TManagedArray<FTransform>&, const TManagedArray<int32>&, const TManagedArray<FTransform>&, TArray<FTransform>&);
	template void CHAOS_API GlobalMatrices<FMatrix>(const TManagedArray<FTransform>&, const TManagedArray<int32>&, const TManagedArray<FTransform>&, TArray<FMatrix>&);

	template void CHAOS_API GlobalMatrices<FTransform, FTransform>(const TManagedArray<FTransform>&, const TManagedArray<int32>&, TArray<FTransform>&);
	template void CHAOS_API GlobalMatrices<FMatrix, FTransform>(const TManagedArray<FTransform>&, const TManagedArray<int32>&, TArray<FMatrix>&);
	template void CHAOS_API GlobalMatrices<FTransform, FTransform3f>(const TManagedArray<FTransform3f>&, const TManagedArray<int32>&, TArray<FTransform>&);
	template void CHAOS_API GlobalMatrices<FMatrix, FTransform3f>(const TManagedArray<FTransform3f>&, const TManagedArray<int32>&, TArray<FMatrix>&);
	template void CHAOS_API GlobalMatrices<FTransform3f, FTransform3f>(const TManagedArray<FTransform3f>&, const TManagedArray<int32>&, TArray<FTransform3f>&);
	template void CHAOS_API GlobalMatrices<FTransform3f, FTransform>(const TManagedArray<FTransform>&, const TManagedArray<int32>&, TArray<FTransform3f>&);


	void FloodForOverlappedPairs(int Level, int32 BoneIndex, TMap<int32, int32> &BoneToGroup, const TManagedArray<int32>& Levels, const TMap<int32, FBox>& BoundingBoxes, TSet<TTuple<int32, int32>>& OutOverlappedPairs)
	{
		if (Levels[BoneIndex] != Level)
		{
			return;
		}

		if (BoneToGroup[BoneIndex] > 0)
		{
			return;
		}

		BoneToGroup[BoneIndex] = 1;

		const FBox& CurrentBoneBounds = BoundingBoxes[BoneIndex];

		for (auto &BoneGroup : BoneToGroup)
		{
			if (BoneGroup.Value < 1 && BoneGroup.Key != BoneIndex) //ungrouped
			{
				const FBox& BoneBounds = BoundingBoxes[BoneGroup.Key];
				if (CurrentBoneBounds.Intersect(BoneBounds))
				{
					auto TupleA = MakeTuple(BoneIndex, BoneGroup.Key);
					auto TupleB = MakeTuple(BoneGroup.Key, BoneIndex);

					if (!OutOverlappedPairs.Contains(TupleA) && !OutOverlappedPairs.Contains(TupleB))
					{
						OutOverlappedPairs.Add(TupleA);
					}

					FloodForOverlappedPairs(Level, BoneGroup.Key, BoneToGroup, Levels, BoundingBoxes, OutOverlappedPairs);
				}
			}
		}
	}

	void GetOverlappedPairs(FGeometryCollection* GeometryCollection, int Level, TSet<TTuple<int32, int32>>& OutOverlappedPairs)
	{
		if (Level > 0)
		{
			const TManagedArray<int32>& Parents = GeometryCollection->Parent;
			if (!ensure(GeometryCollection->HasAttribute("Level", FGeometryCollection::TransformGroup)))
			{
				return;
			}
			const TManagedArray<int32>& Levels = GeometryCollection->GetAttribute<int32>("Level", FGeometryCollection::TransformGroup);

			TArray<FTransform> Transforms;
			GeometryCollectionAlgo::GlobalMatrices(GeometryCollection->Transform, Parents, Transforms);

			TArray<int32> TransformToGeometry;
			GeometryCollectionAlgo::BuildTransformGroupToGeometryGroupMap(*GeometryCollection, TransformToGeometry);

			const TManagedArray<FBox>& BoundingBoxes = GeometryCollection->BoundingBox;

			TMap<int32, int32> BoneToGroup;
			TMap<int32, FBox> WorldBounds;
			for (int32 Element = 0, NumElement = Levels.Num(); Element < NumElement; ++Element)
			{
				if (Levels[Element] == Level)
				{
					const FBox& BoneBounds = BoundingBoxes[TransformToGeometry[Element]];
					BoneToGroup.Add(Element, 0);
					WorldBounds.Add(Element, BoneBounds.TransformBy(Transforms[Element]));
				}
			}

			for (auto &Element : BoneToGroup)
			{
				if (Element.Value < 1)
				{
					FloodForOverlappedPairs(Level, Element.Key, BoneToGroup, Levels, WorldBounds, OutOverlappedPairs);
				}
			}
		}
	}

	void PrepareForSimulation(FGeometryCollection* GeometryCollection, bool CenterAtOrigin/*=true*/)
	{
		check(GeometryCollection);
	}

	void ReCenterGeometryAroundCentreOfMass(FGeometryCollection* GeometryCollection, bool CenterAtOrigin/*=true*/)
	{
		check(GeometryCollection);

		TManagedArray<FTransform3f>& Transform = GeometryCollection->Transform;
		if (Transform.Num())
		{
			const TManagedArray<int32>& BoneMap = GeometryCollection->BoneMap;
			TManagedArray<FVector3f>& Vertex = GeometryCollection->Vertex;

			TArray<int32> SurfaceParticlesCount;
			SurfaceParticlesCount.AddZeroed(GeometryCollection->NumElements(FGeometryCollection::TransformGroup));

			TArray<FVector3f> CenterOfMass;
			CenterOfMass.AddZeroed(GeometryCollection->NumElements(FGeometryCollection::TransformGroup));

			for (int i = 0; i < Vertex.Num(); i++)
			{
				int32 ParticleIndex = BoneMap[i];
				SurfaceParticlesCount[ParticleIndex]++;
				CenterOfMass[ParticleIndex] += Vertex[i];
			}

			FVector3f CombinedCenterOfMassWorld(ForceInitToZero);
			for (int i = 0; i < Transform.Num(); i++)
			{
				if (SurfaceParticlesCount[i])
				{
					CenterOfMass[i] /= static_cast<float>(SurfaceParticlesCount[i]);

					FTransform3f Tmp((FVector3f)CenterOfMass[i]);

					// Translate back to original object space position (because vertex position will be centered at the origin), 
					// then apply the original parent transform.  This ensures the pivot remains the same
					Transform[i] = Tmp * Transform[i];
					CombinedCenterOfMassWorld += Transform[i].GetTranslation();
				}
			}

			CombinedCenterOfMassWorld /= static_cast<float>(Transform.Num());

			for (int i = 0; i < Vertex.Num(); i++)
			{
				int32 ParticleIndex = BoneMap[i];
				Vertex[i] -= CenterOfMass[ParticleIndex];
			}

			if (CenterAtOrigin)
			{
				for (int i = 0; i < Transform.Num(); i++)
				{
					FTransform3f Tmp(-CombinedCenterOfMassWorld);

					// Apply the parent transform, then center at the origin
					Transform[i] = Transform[i] * Tmp;
				}
			}
		}
	}

	void FindOpenBoundaries(const FGeometryCollection* GeometryCollection, const float CoincidentVertexTolerance, TArray<TArray<TArray<int32>>> &BoundaryVertexIndices)
	{
		check(GeometryCollection);
		int32 NumGeometries = GeometryCollection->NumElements(FGeometryCollection::GeometryGroup);
		BoundaryVertexIndices.SetNum(NumGeometries);

		TMap<int32, int32> CoincidentVerticesMap;
		TSet<int32> VertexToDeleteSet_unused; // not needed for this algorithm
		ComputeCoincidentVertices(GeometryCollection, CoincidentVertexTolerance, CoincidentVerticesMap, VertexToDeleteSet_unused);

		for (int32 GeometryIdx = 0; GeometryIdx < NumGeometries; GeometryIdx++)
		{
			// Swap VertexIndex in Indices array
			const TManagedArray<FIntVector>& Indices = GeometryCollection->Indices;
			TMultiMap<int32, int32> OpenEdges;
			auto MapCoincident = [&](int32 VtxIndex)
			{
				if (CoincidentVerticesMap.Contains(VtxIndex))
				{
					return CoincidentVerticesMap[VtxIndex];
				}
				else
				{
					return VtxIndex;
				}
			};
			auto AddEdge = [&](int32 a, int32 b)
			{
				a = MapCoincident(a);
				b = MapCoincident(b);
				if (!OpenEdges.RemoveSingle(b, a))
				{
					OpenEdges.Add(a, b);
				}
			};
			int32 FaceStart = GeometryCollection->FaceStart[GeometryIdx];
			int32 FaceEnd = FaceStart + GeometryCollection->FaceCount[GeometryIdx];
			int32 NumFaces = GeometryCollection->NumElements(FGeometryCollection::FacesGroup);
			for (int32 IdxFace = FaceStart; IdxFace < FaceEnd; ++IdxFace)
			{
				AddEdge(Indices[IdxFace].X, Indices[IdxFace].Y);
				AddEdge(Indices[IdxFace].Y, Indices[IdxFace].Z);
				AddEdge(Indices[IdxFace].Z, Indices[IdxFace].X);
			}
			while (true)
			{
				const auto &EdgeIter = OpenEdges.CreateConstIterator();
				if (!EdgeIter)
				{
					break;
				}
				int32 Start = EdgeIter.Key();
				int32 Walk = EdgeIter.Value();
				TArray<int32> &Boundary = BoundaryVertexIndices[GeometryIdx].Emplace_GetRef();
				Boundary.Add(Start);
				OpenEdges.RemoveSingle(Start, Walk);
				while (Walk != Start)
				{
					Boundary.Add(Walk);
					int32 Next = OpenEdges.FindChecked(Walk);
					OpenEdges.RemoveSingle(Walk, Next);
					Walk = Next;
				}
			}
		}
	}

	void TriangulateBoundaries(FGeometryCollection* GeometryCollection, const TArray<TArray<TArray<int32>>> &BoundaryVertexIndices, bool bWoundClockwise, float MinTriangleAreaSq)
	{
		check(GeometryCollection);
		int32 NumGeometries = GeometryCollection->NumElements(FGeometryCollection::GeometryGroup);
		check(BoundaryVertexIndices.Num() == NumGeometries);

		TArray<TArray<FIntVector>> Faces;
		Faces.SetNum(NumGeometries);

		TMap<TPair<int32, int32>, int32> FaceCountPerEdge;
		
		auto AddTriToFaceCountPerEdge = [&FaceCountPerEdge](const FIntVector &Face)
		{
			auto AddEdge = [&FaceCountPerEdge](int32 A, int32 B)
			{
				FaceCountPerEdge.FindOrAdd(TPair<int32, int32>(FMath::Min(A, B), FMath::Max(A, B)))++;
			};
			AddEdge(Face.X, Face.Y);
			AddEdge(Face.Y, Face.Z);
			AddEdge(Face.Z, Face.X);
		};

		auto CandidateTriWouldMakeNonManifoldEdge = [&FaceCountPerEdge](int32 A, int32 B, int32 C)
		{
			auto GetCount = [&FaceCountPerEdge](int32 InnerA, int32 InnerB)
			{
				int32 *Count = FaceCountPerEdge.Find(TPair<int32, int32>(FMath::Min(InnerA, InnerB), FMath::Max(InnerA, InnerB)));
				return Count ? *Count : 0;
			};
			return GetCount(A, B) > 1 || GetCount(B, C) > 1 || GetCount(C, A) > 1;
		};

		for (const FIntVector &Face : GeometryCollection->Indices)
		{
			AddTriToFaceCountPerEdge(Face);
		}

		for (int32 GeometryIdx = 0; GeometryIdx < NumGeometries; GeometryIdx++)
		{
			const TArray<TArray<int32>> &GeomBoundaries = BoundaryVertexIndices[GeometryIdx];
			TArray<FIntVector> &GeomFaces = Faces[GeometryIdx];
			// for v0 let's just put a fan here
			for (const TArray<int32>& Boundary : GeomBoundaries)
			{
				if (Boundary.Num() < 3)
				{
					continue;
				}

				// Note: technically this is 4x the squared area, but matches how Chaos generally filters degenerate triangles
				auto GetAreaSq = [GeometryCollection](int32 A, int32 B, int32 C) -> double
				{
					FVector p10(GeometryCollection->Vertex[B] - GeometryCollection->Vertex[A]);
					FVector p20(GeometryCollection->Vertex[C] - GeometryCollection->Vertex[A]);
					FVector Cross = FVector::CrossProduct(p20, p10);
					return Cross.SizeSquared();
				};

				// Try to find a start index that isn't a locally degenerate/collinear chain (if we have a positive degenerate tri threshold)
				int32 UseStartIdx = 0;
				if (MinTriangleAreaSq > 0 && Boundary.Num() > 3)
				{
					int32 NumLocalNonCollinear = 0;
					for (int32 Prev2 = Boundary.Num() - 2, Prev = Boundary.Num() - 1, Idx = 0; Idx < Boundary.Num(); Prev2 = Prev, Prev = Idx++)
					{
						if (GetAreaSq(Boundary[Prev2], Boundary[Prev], Boundary[Idx]) < MinTriangleAreaSq)
						{
							NumLocalNonCollinear = 0;
						}
						else
						{
							++NumLocalNonCollinear;
						}
						if (NumLocalNonCollinear == 3)
						{
							UseStartIdx = Prev2;
							break;
						}
					}
				}

				int32 First = Boundary[UseStartIdx];
				
				for (int32 BoundaryIdx = 1; BoundaryIdx + 1 < Boundary.Num(); BoundaryIdx++)
				{
					int32 A = First, B = Boundary[(UseStartIdx + BoundaryIdx) % Boundary.Num()], C = Boundary[(UseStartIdx + BoundaryIdx + 1) % Boundary.Num()];

					if (MinTriangleAreaSq > 0)
					{
						if (GetAreaSq(A, B, C) < MinTriangleAreaSq)
						{
							continue;
						}
					}

					if (CandidateTriWouldMakeNonManifoldEdge(A, B, C))
					{
						continue;
					}

					if (bWoundClockwise)
					{
						GeomFaces.Add(FIntVector(A, C, B));
					}
					else
					{
						GeomFaces.Add(FIntVector(A, B, C));
					}
					
				}
			}
		}

		AddFaces(GeometryCollection, Faces);
	}

	void AddFaces(FGeometryCollection* GeometryCollection, const TArray<TArray<FIntVector>> &AddFaces)
	{
		check(GeometryCollection);
		int32 NumGeometries = GeometryCollection->NumElements(FGeometryCollection::GeometryGroup);
		check(AddFaces.Num() == NumGeometries);

		int32 AddCount = 0;
		for (const TArray<FIntVector> &GeomFaces : AddFaces)
		{
			AddCount += GeomFaces.Num();
		}
		int32 OldNumFaces = GeometryCollection->NumElements(FGeometryCollection::FacesGroup);
		GeometryCollection->AddElements(AddCount, FGeometryCollection::FacesGroup);
		int32 NewNumFaces = GeometryCollection->NumElements(FGeometryCollection::FacesGroup);

		TArray<int32> ShiftedOrder;
		ShiftedOrder.SetNum(NewNumFaces);
		int32 ShiftedIdx = 0;
		int32 NewSpaceIdx = OldNumFaces;
		for (int32 GeometryIdx = 0; GeometryIdx < NumGeometries; GeometryIdx++)
		{
			int32 OldStart = GeometryCollection->FaceStart[GeometryIdx];
			int32 NewStart = ShiftedIdx;
			int32 OldEnd = OldStart + GeometryCollection->FaceCount[GeometryIdx];
			for (int32 OldFaceIdx = OldStart; OldFaceIdx < OldEnd; OldFaceIdx++)
			{
				ShiftedOrder[ShiftedIdx++] = OldFaceIdx;
			}
			int32 NumToAdd = AddFaces[GeometryIdx].Num();
			for (int32 AddFaceIdx=0; AddFaceIdx < NumToAdd; AddFaceIdx++)
			{
				GeometryCollection->Indices[NewSpaceIdx] = AddFaces[GeometryIdx][AddFaceIdx];
				GeometryCollection->Visible[NewSpaceIdx] = true;
				// just copy material from one of the faces in the group
				GeometryCollection->MaterialIndex[NewSpaceIdx] = GeometryCollection->MaterialIndex[OldStart];
				GeometryCollection->MaterialID[NewSpaceIdx] = GeometryCollection->MaterialID[OldStart];
				// also copy internal status from the existing faces
				GeometryCollection->Internal[NewSpaceIdx] = GeometryCollection->Internal[OldStart];
				ShiftedOrder[ShiftedIdx++] = NewSpaceIdx;
				NewSpaceIdx++;
			}
			GeometryCollection->FaceCount[GeometryIdx] += NumToAdd;
		}
		GeometryCollection->ReorderElements(FGeometryCollection::FacesGroup, ShiftedOrder);
	}

	void ResizeGeometries(FGeometryCollection* GeometryCollection, const TArray<int32>& FaceCounts, const TArray<int32>& VertexCounts, bool bDoValidation)
	{
		check(GeometryCollection);
		int32 NumGeometries = GeometryCollection->NumElements(FGeometryCollection::GeometryGroup);
		check(FaceCounts.Num() == NumGeometries);
		check(VertexCounts.Num() == NumGeometries);

		int32 NewNumFaces = 0, NewNumVertices = 0;
		TArray<int32> UnusedFaces, UnusedVertices;

		auto AddRange = [](TArray<int32>& OutArray, int32 StartIncl, int32 EndExcl)
		{
			for (int32 Idx = StartIncl; Idx < EndExcl; Idx++)
			{
				OutArray.Add(Idx);
			}
		};

		auto CountElements = [&AddRange](int32 OldTotal, const TArray<int32>& NewCounts, const TManagedArray<int32>& OldCounts, const TManagedArray<int32>& OldStarts, int32& OutTotal, TArray<int32>& OutUnused)
		{
			OutTotal = 0;
			OutUnused.Reset();
			for (int32 Idx = 0; Idx < NewCounts.Num(); Idx++)
			{
				OutTotal += NewCounts[Idx];
				if (OldCounts[Idx] > NewCounts[Idx])
				{
					AddRange(OutUnused, OldStarts[Idx] + NewCounts[Idx], OldStarts[Idx] + OldCounts[Idx]);
				}
			}
			if (OutTotal > OldTotal)
			{
				AddRange(OutUnused, OldTotal, OutTotal);
			}
		};

		int32 OldNumFaces = GeometryCollection->NumElements(FGeometryCollection::FacesGroup);
		int32 OldNumVertices = GeometryCollection->NumElements(FGeometryCollection::VerticesGroup);
		CountElements(OldNumFaces, FaceCounts, GeometryCollection->FaceCount, GeometryCollection->FaceStart, NewNumFaces, UnusedFaces);
		CountElements(OldNumVertices, VertexCounts, GeometryCollection->VertexCount, GeometryCollection->VertexStart, NewNumVertices, UnusedVertices);

		// add elements at the end if needed
		if (NewNumFaces > OldNumFaces)
		{
			GeometryCollection->AddElements(NewNumFaces - OldNumFaces, FGeometryCollection::FacesGroup);
			// fill in end faces with dummy values, rather than leaving them uninitialized (to avoid breaking things on later reordering call)
			int LastVertex = OldNumVertices > 0 ? OldNumVertices - 1 : 0;
			FIntVector EndFace(LastVertex, LastVertex, LastVertex);
			for (int Idx = OldNumFaces; Idx < NewNumFaces; Idx++)
			{
				GeometryCollection->Indices[Idx] = EndFace;
			}
		}
		if (NewNumVertices > OldNumVertices)
		{
			GeometryCollection->AddElements(NewNumVertices - OldNumVertices, FGeometryCollection::VerticesGroup);
		}
		
		int32 NumFaces = GeometryCollection->NumElements(FGeometryCollection::FacesGroup);
		int32 NumVertices = GeometryCollection->NumElements(FGeometryCollection::VerticesGroup);

		auto ComputeNewOrder = [&AddRange](int32 Total, const TArray<int32>& NewCounts, const TManagedArray<int32>& OldCounts, const TManagedArray<int32>& OldStarts, TArray<int32>& Unused, TArray<int32>& OutNewOrder)
		{
			OutNewOrder.Reset();
			OutNewOrder.Reserve(Total);
			int32 UnusedIdx = 0;
			for (int32 GeometryIdx = 0; GeometryIdx < OldStarts.Num(); GeometryIdx++)
			{
				if (NewCounts[GeometryIdx] < OldCounts[GeometryIdx]) // Geometry shrunk
				{
					AddRange(OutNewOrder, OldStarts[GeometryIdx], OldStarts[GeometryIdx] + NewCounts[GeometryIdx]);
				}
				else // Geometry grew
				{
					AddRange(OutNewOrder, OldStarts[GeometryIdx], OldStarts[GeometryIdx] + OldCounts[GeometryIdx]);
					int32 GrowAmt = NewCounts[GeometryIdx] - OldCounts[GeometryIdx];
					for (int32 Idx = 0; Idx < GrowAmt; Idx++)
					{
						OutNewOrder.Add(Unused[UnusedIdx++]);
					}
				}
			}
			while (OutNewOrder.Num() < Total)
			{
				OutNewOrder.Add(Unused[UnusedIdx++]);
			}
		};

		TArray<int32> NewFaceOrder, NewVertexOrder;
		ComputeNewOrder(NumFaces, FaceCounts, GeometryCollection->FaceCount, GeometryCollection->FaceStart, UnusedFaces, NewFaceOrder);
		ComputeNewOrder(NumVertices, VertexCounts, GeometryCollection->VertexCount, GeometryCollection->VertexStart, UnusedVertices, NewVertexOrder);
		GeometryCollection->ReorderElements(FGeometryCollection::VerticesGroup, NewVertexOrder);
		GeometryCollection->ReorderElements(FGeometryCollection::FacesGroup, NewFaceOrder);

		// fix face/vertex counts and vertex->transform (bone) map
		for (int32 GeometryIdx = 0; GeometryIdx < NumGeometries; GeometryIdx++)
		{
			int32 TransformIdx = GeometryCollection->TransformIndex[GeometryIdx];
			GeometryCollection->FaceCount[GeometryIdx] = FaceCounts[GeometryIdx];
			GeometryCollection->VertexCount[GeometryIdx] = VertexCounts[GeometryIdx];
			int32 VertexStart = GeometryCollection->VertexStart[GeometryIdx];
			int32 VertexEnd = VertexStart + GeometryCollection->VertexCount[GeometryIdx];
			for (int32 VertexIdx = VertexStart; VertexIdx < VertexEnd; VertexIdx++)
			{
				GeometryCollection->BoneMap[VertexIdx] = TransformIdx;
			}
		}
		for (int32 GeometryIdx = 0; GeometryIdx < NumGeometries; GeometryIdx++)
		{
			int32 TransformIdx = GeometryCollection->TransformIndex[GeometryIdx];
			// the vertex remapping can leave faces that still refer to 'deleted' vertices
			// these faces are then pointing to vertices that aren't in the same geometry
			// the intent is that all resized geometry will be re-written by the caller, afterwards
			// but leaving these broken faces is dangerous and prevents validation in the meantime
			// so we 'fix' them by writing a degenerate (but w/in geo) face in these cases
			int32 FaceStart = GeometryCollection->FaceStart[GeometryIdx];
			int32 FaceEnd = FaceStart + GeometryCollection->FaceCount[GeometryIdx];
			int32 VertexStart = GeometryCollection->VertexStart[GeometryIdx];
			FIntVector ReplacementFace(VertexStart, VertexStart, VertexStart);
			for (int32 FaceIdx = FaceStart; FaceIdx < FaceEnd; FaceIdx++)
			{
				FIntVector& Face = GeometryCollection->Indices[FaceIdx];
				bool bFaceUsesDeletedVertex = false;
				for (int SubIdx = 0; SubIdx < 3; SubIdx++)
				{
					int32 VertIdx = Face[SubIdx];
					bFaceUsesDeletedVertex |=
						(GeometryCollection->BoneMap[VertIdx] != TransformIdx) |
						(VertIdx >= NewNumVertices);
				}
				if (bFaceUsesDeletedVertex)
				{
					Face = ReplacementFace;
				}
			}
		}

		FManagedArrayCollection::FProcessingParameters ProcessingParams;
		ProcessingParams.bDoValidation = bDoValidation;

		// remove trailing elements if needed
		if (NewNumVertices < OldNumVertices)
		{
			TArray<int32> ToDelete;
			AddRange(ToDelete, NewNumVertices, OldNumVertices);
			GeometryCollection->RemoveElements(FGeometryCollection::VerticesGroup, ToDelete, ProcessingParams);
		}
		if (NewNumFaces < OldNumFaces)
		{
			TArray<int32> ToDelete;
			AddRange(ToDelete, NewNumFaces, OldNumFaces);
			GeometryCollection->RemoveElements(FGeometryCollection::FacesGroup, ToDelete, ProcessingParams);
		}

		if (bDoValidation)
		{
			ensure(GeometryCollection->HasContiguousFaces());
			ensure(GeometryCollection->HasContiguousVertices());
			ensure(GeometryCollectionAlgo::HasValidGeometryReferences(GeometryCollection));
		}
	}

	DEFINE_LOG_CATEGORY_STATIC(LogGeometryCollectionClean, Verbose, All);

	void ComputeCoincidentVertices(const FGeometryCollection* GeometryCollection, const float Tolerance, TMap<int32, int32>& CoincidentVerticesMap, TSet<int32>& VertexToDeleteSet)
	{
		check(GeometryCollection);

		const TManagedArray<FVector3f>& VertexArray = GeometryCollection->Vertex;
		const TManagedArray<int32>& BoneMapArray = GeometryCollection->BoneMap;
		const TManagedArray<int32>& TransformIndexArray = GeometryCollection->TransformIndex;
		int32 NumVertices = GeometryCollection->NumElements(FGeometryCollection::VerticesGroup);
		int32 NumGeometries = GeometryCollection->NumElements(FGeometryCollection::GeometryGroup);

		float ToleranceSquared = Tolerance * Tolerance;

		FCriticalSection Mutex;

		ParallelFor(NumGeometries, [&](int32 IdxGeometry)
		{
			TMap<int32, int32> LocalCoincidentVerticesMap;
			TSet<int32> LocalVertexToDeleteSet;
			int32 TransformIndex = TransformIndexArray[IdxGeometry];
			for (int32 IdxVertex = 0; IdxVertex < NumVertices; ++IdxVertex)
			{
				if (BoneMapArray[IdxVertex] == TransformIndex)
				{
					if (!LocalVertexToDeleteSet.Contains(IdxVertex))
					{
						const FVector3f& Vertex = VertexArray[IdxVertex];
						for (int32 IdxOtherVertex = 0; IdxOtherVertex < NumVertices; ++IdxOtherVertex)
						{
							if (BoneMapArray[IdxOtherVertex] == TransformIndex)
							{
								if ((IdxVertex != IdxOtherVertex) && !LocalVertexToDeleteSet.Contains(IdxOtherVertex))
								{
									const FVector3f& OtherVertex = VertexArray[IdxOtherVertex];
									if ((Vertex - OtherVertex).SizeSquared() < ToleranceSquared)
									{
										LocalVertexToDeleteSet.Add(IdxOtherVertex);
										LocalCoincidentVerticesMap.Add(IdxOtherVertex, IdxVertex);
									}
								}
							}
						}
					}
				}
			}
			if (LocalVertexToDeleteSet.Num())
			{
				Mutex.Lock();
				CoincidentVerticesMap.Append(LocalCoincidentVerticesMap);
				VertexToDeleteSet.Append(LocalVertexToDeleteSet);
				Mutex.Unlock();
			}
		});
	}


	void DeleteCoincidentVertices(FGeometryCollection* GeometryCollection, float Tolerance)
	{
		check(GeometryCollection);

		TSet<int32> VertexToDeleteSet;
		TMap<int32, int32> CoincidentVerticesMap;
		ComputeCoincidentVertices(GeometryCollection, Tolerance, CoincidentVerticesMap, VertexToDeleteSet);

		// Swap VertexIndex in Indices array
		TManagedArray<FIntVector>& IndicesArray = GeometryCollection->Indices;
		int32 NumFaces = GeometryCollection->NumElements(FGeometryCollection::FacesGroup);
		for (int32 IdxFace = 0; IdxFace < NumFaces; ++IdxFace)
		{
			if (CoincidentVerticesMap.Contains(IndicesArray[IdxFace].X))
			{
				IndicesArray[IdxFace].X = CoincidentVerticesMap[IndicesArray[IdxFace].X];
			}
			if (CoincidentVerticesMap.Contains(IndicesArray[IdxFace].Y))
			{
				IndicesArray[IdxFace].Y = CoincidentVerticesMap[IndicesArray[IdxFace].Y];
			}
			if (CoincidentVerticesMap.Contains(IndicesArray[IdxFace].Z))
			{
				IndicesArray[IdxFace].Z = CoincidentVerticesMap[IndicesArray[IdxFace].Z];
			}
		}

		// Delete vertices
		TArray<int32> DelList = VertexToDeleteSet.Array();
		DelList.Sort();
		GeometryCollection->RemoveElements(FGeometryCollection::VerticesGroup, DelList);
	}

	void ComputeZeroAreaFaces(const FGeometryCollection* GeometryCollection, const float Tolerance, TSet<int32>& FaceToDeleteSet)
	{
		check(GeometryCollection);

		const TManagedArray<FVector3f>& VertexArray = GeometryCollection->Vertex;
		const TManagedArray<FIntVector>& IndicesArray = GeometryCollection->Indices;
		const TManagedArray<int32>& BoneMapArray = GeometryCollection->BoneMap;

		int32 NumVertices = GeometryCollection->NumElements(FGeometryCollection::VerticesGroup);
		int32 NumFaces = GeometryCollection->NumElements(FGeometryCollection::FacesGroup);

		for (int32 IdxFace = 0; IdxFace < NumFaces; ++IdxFace)
		{
			int32 TransformIndex = BoneMapArray[IndicesArray[IdxFace][0]];
			if (GeometryCollection->IsGeometry(TransformIndex) && !GeometryCollection->IsClustered(TransformIndex))
			{
				FVector3f Vertex0 = VertexArray[IndicesArray[IdxFace][0]];
				FVector3f Vertex1 = VertexArray[IndicesArray[IdxFace][1]];
				FVector3f Vertex2 = VertexArray[IndicesArray[IdxFace][2]];

				float Area = 0.5f * ((Vertex0 - Vertex1) ^ (Vertex0 - Vertex2)).Size();
				if (Area < Tolerance)
				{
					FaceToDeleteSet.Add(IdxFace);
				}
			}
		}
	}

	void DeleteZeroAreaFaces(FGeometryCollection* GeometryCollection, float Tolerance)
	{
		check(GeometryCollection);

		TSet<int32> FaceToDeleteSet;
		ComputeZeroAreaFaces(GeometryCollection, Tolerance, FaceToDeleteSet);

		TArray<int32> DelList = FaceToDeleteSet.Array();
		DelList.Sort();
		GeometryCollection->RemoveElements(FGeometryCollection::FacesGroup, DelList);
	}

	void ComputeHiddenFaces(const FGeometryCollection* GeometryCollection, TSet<int32>& FaceToDeleteSet)
	{
		check(GeometryCollection);

		const TManagedArray<FIntVector>& IndicesArray = GeometryCollection->Indices;
		const TManagedArray<bool>& VisibleArray = GeometryCollection->Visible;
		const TManagedArray<int32>& BoneMapArray = GeometryCollection->BoneMap;

		int32 NumFaces = GeometryCollection->NumElements(FGeometryCollection::FacesGroup);

		for (int32 IdxFace = 0; IdxFace < NumFaces; ++IdxFace)
		{
			int32 TransformIndex = BoneMapArray[IndicesArray[IdxFace][0]];
			if (GeometryCollection->IsGeometry(TransformIndex) && !GeometryCollection->IsClustered(TransformIndex))
			{
				if (!VisibleArray[IdxFace])
				{
					FaceToDeleteSet.Add(IdxFace);
				}
			}
		}
	}

	void DeleteHiddenFaces(FGeometryCollection* GeometryCollection)
	{
		check(GeometryCollection);

		TSet<int32> FaceToDeleteSet;
		ComputeHiddenFaces(GeometryCollection, FaceToDeleteSet);

		TArray<int32> DelList = FaceToDeleteSet.Array();
		DelList.Sort();
		GeometryCollection->RemoveElements(FGeometryCollection::FacesGroup, DelList);
	}

	void ComputeStaleVertices(const FGeometryCollection* GeometryCollection, TSet<int32>& VertexToDeleteSet)
	{
		check(GeometryCollection);

		const TManagedArray<FVector3f>& VertexArray = GeometryCollection->Vertex;
		const TManagedArray<int32>& BoneMapArray = GeometryCollection->BoneMap;
		const TManagedArray<int32>& TransformIndexArray = GeometryCollection->TransformIndex;
		const TManagedArray<FIntVector>& IndicesArray = GeometryCollection->Indices;

		int32 NumVertices = GeometryCollection->NumElements(FGeometryCollection::VerticesGroup);
		int32 NumFaces = GeometryCollection->NumElements(FGeometryCollection::FacesGroup);

		TArray<int32> VertexInFaceArray;
		VertexInFaceArray.Init(0, NumVertices);
		for (int32 IdxFace = 0; IdxFace < NumFaces; ++IdxFace)
		{
			VertexInFaceArray[IndicesArray[IdxFace].X]++;
			VertexInFaceArray[IndicesArray[IdxFace].Y]++;
			VertexInFaceArray[IndicesArray[IdxFace].Z]++;
		}

		if (const TManagedArray<FIntVector4> *TetArrayPtr = 
			GeometryCollection->FindAttributeTyped<FIntVector4>("Tetrahedron", "Tetrahedral"))
		{
			int32 NumTetrahedron = GeometryCollection->NumElements("Tetrahedral");
			const TManagedArray<FIntVector4>& TetArray = *TetArrayPtr;
			for (int32 TetIdx = 0; TetIdx < NumTetrahedron; ++TetIdx)
			{
				VertexInFaceArray[TetArray[TetIdx].X]++;
				VertexInFaceArray[TetArray[TetIdx].Y]++;
				VertexInFaceArray[TetArray[TetIdx].Z]++;
				VertexInFaceArray[TetArray[TetIdx].W]++;
			}
		}

		for (int32 IdxVertex = 0; IdxVertex < NumVertices; ++IdxVertex)
		{
			if (VertexInFaceArray[IdxVertex] == 0)
			{
				VertexToDeleteSet.Add(IdxVertex);
			}
		}
	}

	void DeleteStaleVertices(FGeometryCollection* GeometryCollection)
	{
		check(GeometryCollection);

		TSet<int32> VertexToDeleteSet;
		ComputeStaleVertices(GeometryCollection, VertexToDeleteSet);

		TArray<int32> DelList = VertexToDeleteSet.Array();
		DelList.Sort();
		GeometryCollection->RemoveElements(FGeometryCollection::VerticesGroup, DelList);
	}

	void ComputeEdgeInFaces(const FGeometryCollection* GeometryCollection, TMap<FFaceEdge, int32>& FaceEdgeMap)
	{
		check(GeometryCollection);

		const TManagedArray<FIntVector>& IndicesArray = GeometryCollection->Indices;

		int32 NumFaces = GeometryCollection->NumElements(FGeometryCollection::FacesGroup);

		for (int32 IdxFace = 0; IdxFace < NumFaces; ++IdxFace)
		{
			for (int32 Idx = 0; Idx < 3; Idx++)
			{
				int32 VertexIndex1 = IndicesArray[IdxFace][Idx];
				int32 VertexIndex2 = IndicesArray[IdxFace][(Idx + 1) % 3];
				FFaceEdge Edge{ FMath::Min(VertexIndex1, VertexIndex2),
								FMath::Max(VertexIndex1, VertexIndex2) };
				if (FaceEdgeMap.Contains(Edge))
				{
					FaceEdgeMap[Edge]++;
				}
				else
				{
					FaceEdgeMap.Add(Edge, 1);
				}
			}
		}
	}

	void PrintStatistics(const FGeometryCollection* GeometryCollection)
	{
		check(GeometryCollection);

		int32 NumVertices = GeometryCollection->NumElements(FGeometryCollection::VerticesGroup);
		int32 NumFaces = GeometryCollection->NumElements(FGeometryCollection::FacesGroup);
		int32 NumGeometries = GeometryCollection->NumElements(FGeometryCollection::GeometryGroup);
		int32 NumTransforms = GeometryCollection->NumElements(FGeometryCollection::TransformGroup);
		int32 NumBreakings = GeometryCollection->NumElements(FGeometryCollection::BreakingGroup);

		FString Buffer;
		Buffer += FString::Printf(TEXT("\n\n"));
		Buffer += FString::Printf(TEXT("------------------------------------------------------------\n"));
		Buffer += FString::Printf(TEXT("Number of transforms = %d\n"), NumTransforms);
		Buffer += FString::Printf(TEXT("Number of vertices = %d\n"), NumVertices);
		Buffer += FString::Printf(TEXT("Number of faces = %d\n"), NumFaces);
		Buffer += FString::Printf(TEXT("Number of geometries = %d\n"), NumGeometries);
		Buffer += FString::Printf(TEXT("Number of breakings = %d\n"), NumBreakings);
		Buffer += FString::Printf(TEXT("------------------------------------------------------------\n\n"));
		UE_LOGF(LogGeometryCollectionClean, Log, "%ls", *Buffer);
	}

	bool HasValidFacesFor(const FGeometryCollection* GeometryCollection, int32 GeometryIndex)
	{
		ensure(GeometryIndex < GeometryCollection->NumElements(FGeometryCollection::GeometryGroup));

		int32 FaceStart = GeometryCollection->FaceStart[GeometryIndex];
		int32 FaceCount = GeometryCollection->FaceCount[GeometryIndex];
		int32 FaceEnd = FaceStart + FaceCount;

		// check faces range within number of elements in faces group
		int32 NumFaces = GeometryCollection->NumElements(FGeometryCollection::FacesGroup);
		if (FaceStart >= NumFaces)
			return false;

		if (FaceEnd > NumFaces)
			return false;

		// check faces index into valid elements of vertices group
		int32 VertexStart = GeometryCollection->VertexStart[GeometryIndex];
		int32 VertexCount = GeometryCollection->VertexCount[GeometryIndex];
		int32 VertexEnd = VertexStart + VertexCount;

		const TManagedArray<FIntVector>& Indices = GeometryCollection->Indices;
		for (int FaceIdx = FaceStart; FaceIdx < FaceEnd; FaceIdx++)
		{
			const FIntVector& Face = Indices[FaceIdx];
			for (int Idx = 0; Idx < 3; Idx++)
			{
				if (Face[Idx] < VertexStart || Face[Idx] >= VertexEnd)
				{
					return false;
				}

				int32 BoneAndTransformIndex = GeometryCollection->BoneMap[Face[Idx]];
				int32 TestGeometryIndex = GeometryCollection->TransformToGeometryIndex[BoneAndTransformIndex];

				if (GeometryIndex != TestGeometryIndex)
					return false;
			}
		}

		return true;
	}

	bool HasValidIndicesFor(const FGeometryCollection* GeometryCollection, int32 GeometryIndex)
	{
		ensure(GeometryIndex < GeometryCollection->NumElements(FGeometryCollection::GeometryGroup));

		int32 VertexStart = GeometryCollection->VertexStart[GeometryIndex];
		int32 VertexCount = GeometryCollection->VertexCount[GeometryIndex];
		int32 VertexEnd = VertexStart + VertexCount;
		
		int32 NumVertices = GeometryCollection->NumElements(FGeometryCollection::VerticesGroup);

		if (VertexStart >= NumVertices)
			return false;

		if (VertexEnd > NumVertices)
			return false;

		for (int VertIdx = VertexStart; VertIdx < VertexEnd; VertIdx++)
		{
			int32 BoneAndTransformIndex = GeometryCollection->BoneMap[VertIdx];
			int32 TestGeometryIndex = GeometryCollection->TransformToGeometryIndex[BoneAndTransformIndex];

			if (GeometryIndex != TestGeometryIndex)
				return false;
		}

		return true;
	}

	bool HasInvalidIndicesFor(const FGeometryCollection* GeometryCollection, int32 GeometryIndex)
	{
		ensure(GeometryIndex < GeometryCollection->NumElements(FGeometryCollection::GeometryGroup));

		int32 VertexStart = GeometryCollection->VertexStart[GeometryIndex];
		int32 VertexCount = GeometryCollection->VertexCount[GeometryIndex];
		int32 VertexEnd = VertexStart + VertexCount;

		int32 NumVertices = GeometryCollection->NumElements(FGeometryCollection::VerticesGroup);

		if (VertexStart >= NumVertices)
			return true;

		if (VertexEnd > NumVertices)
			return true;

		for (int32 VertIdx = 0 ; VertIdx < NumVertices ; ++VertIdx)
		{
			if (VertIdx >= VertexStart && VertIdx < VertexEnd)
				continue;

			int32 BoneAndTransformIndex = GeometryCollection->BoneMap[VertIdx];
			int32 TestGeometryIndex = GeometryCollection->TransformToGeometryIndex[BoneAndTransformIndex];

			if (GeometryIndex == TestGeometryIndex)
				return true;

		}

		int32 OurTransformIndex = GeometryCollection->TransformIndex[GeometryIndex];
		for (int32 GeomIdx=0, NumGeo = GeometryCollection->TransformIndex.Num(); GeomIdx < NumGeo; ++GeomIdx)
		{
			if (GeomIdx == GeometryIndex)
				continue;

			if (GeometryCollection->TransformIndex[GeomIdx] == OurTransformIndex)
				return true;
		}

		return false;
	}

	bool HasResidualFaces(const FGeometryCollection* GeometryCollection)
	{
		int32 NumFaces = GeometryCollection->NumElements(FGeometryCollection::FacesGroup);
		TArray<bool> IsUsed;
		IsUsed.Init(false, NumFaces);

		for (int32 GeometryIndex = 0, NumGeometry = GeometryCollection->NumElements(FGeometryCollection::GeometryGroup) ; GeometryIndex < NumGeometry ; ++GeometryIndex)
		{
			int32 FaceStart = GeometryCollection->FaceStart[GeometryIndex];
			int32 FaceCount = GeometryCollection->FaceCount[GeometryIndex];
			int32 FaceEnd = FaceStart + FaceCount;

			ensureMsgf(FaceStart < NumFaces, TEXT("Geometry %d has invalid face start index %d"), GeometryIndex, FaceStart);
			ensureMsgf(FaceEnd <= NumFaces, TEXT("Geometry %d has invalid face end index %d"), GeometryIndex, FaceEnd);

			for (int32 Idx = FaceStart; Idx < FaceEnd; Idx++)
			{
				IsUsed[Idx] = true;
			}
		}

		for (bool Used : IsUsed)
		{
			if (Used == false)
				return true;
		}

		return false;
	}

	bool HasResidualIndices(const FGeometryCollection* GeometryCollection)
	{
		int32 NumVertices = GeometryCollection->NumElements(FGeometryCollection::VerticesGroup);
		TArray<bool> IsUsed;
		IsUsed.Init(false, NumVertices);

		for (int32 GeometryIndex = 0, NumGeometry = GeometryCollection->NumElements(FGeometryCollection::GeometryGroup); GeometryIndex < NumGeometry; ++GeometryIndex)
		{
			int32 VertexStart = GeometryCollection->VertexStart[GeometryIndex];
			int32 VertexCount = GeometryCollection->VertexCount[GeometryIndex];
			int32 VertexEnd = VertexStart + VertexCount;

			ensureMsgf(VertexStart < NumVertices, TEXT("Geometry %d has invalid vertex start index %d"), GeometryIndex, VertexStart);
			ensureMsgf(VertexEnd <= NumVertices, TEXT("Geometry %d has invalid vertex end index %d"), GeometryIndex, VertexEnd);

			for (int32 Idx = VertexStart; Idx < VertexEnd; Idx++)
			{
				IsUsed[Idx] = true;
			}
		}

		for (bool Used : IsUsed)
		{
			if (Used == false)
				return true;
		}

		return false;
	}

	bool HasValidGeometryReferences(const FGeometryCollection* GeometryCollection)
	{
		for (int GeometryIndex = 0; GeometryIndex < GeometryCollection->NumElements(FGeometryCollection::GeometryGroup); GeometryIndex++)
		{
			if (!HasValidIndicesFor(GeometryCollection, GeometryIndex))
				return false;

			if (!HasValidFacesFor(GeometryCollection, GeometryIndex))
				return false;

			if (HasInvalidIndicesFor(GeometryCollection, GeometryIndex))
				return false;
		}

		return true;
	}


	TArray<int32> ComputeRecursiveOrder(const FManagedArrayCollection& Collection)
	{
		const int32 NumTransforms = Collection.NumElements(FGeometryCollection::TransformGroup);

		Chaos::Facades::FCollectionHierarchyFacade HierarchyFacade(Collection);
		if (!ensure(HierarchyFacade.IsValid()))
		{
			// We cannot compute a recursive ordering without hierarchy attributes
			TArray<int32> Transforms;
			return Transforms;
		}

		//traverse cluster hierarchy in depth first and record order
		struct FClusterProcessing
		{
			int32 TransformGroupIndex;
			enum
			{
				None,
				VisitingChildren
			} State;

			FClusterProcessing(int32 InIndex) : TransformGroupIndex(InIndex), State(None) {};
		};

		TArray<FClusterProcessing> ClustersToProcess;
		//enqueue all roots
		for (int32 TransformGroupIndex = 0; TransformGroupIndex < NumTransforms; TransformGroupIndex++)
		{
			if (HierarchyFacade.GetParent(TransformGroupIndex) == FGeometryCollection::Invalid)
			{
				ClustersToProcess.Emplace(TransformGroupIndex);
			}
		}

		TArray<int32> TransformOrder;
		TransformOrder.Reserve(NumTransforms);

		while (ClustersToProcess.Num())
		{
			FClusterProcessing CurCluster = ClustersToProcess.Pop();
			const int32 ClusterTransformIdx = CurCluster.TransformGroupIndex;
			if (CurCluster.State == FClusterProcessing::VisitingChildren)
			{
				//children already visited
				TransformOrder.Add(ClusterTransformIdx);
			}
			else
			{
				const TSet<int32>* ClusterChildren = HierarchyFacade.FindChildren(ClusterTransformIdx);
				if (ClusterChildren && ClusterChildren->Num())
				{
					CurCluster.State = FClusterProcessing::VisitingChildren;
					ClustersToProcess.Add(CurCluster);

					//order of children doesn't matter as long as all children appear before parent
					for (int32 ChildIdx : *ClusterChildren)
					{
						ClustersToProcess.Emplace(ChildIdx);
					}
				}
				else
				{
					TransformOrder.Add(ClusterTransformIdx);
				}
			}
		}

		return TransformOrder;
	}

}

