// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionClusteringUtility.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "Containers/Set.h"
#include "Async/ParallelFor.h"
#include "GeometryCollection/Facades/CollectionHierarchyFacade.h"

static int32 ChaosValidateResultsOfEditOperations = 0;
static FAutoConsoleVariableRef CVarChaosStillCheckDistanceThreshold(TEXT("p.fracture.ValidateResultsOfEditOperations"), ChaosValidateResultsOfEditOperations, TEXT("When on this will enable result validation for fracture tool edit operations (can be slow for large geometry collection) [def:0]"));

int32 FGeometryCollectionClusteringUtility::ClusterBonesUnderNewNode(FGeometryCollection* GeometryCollection, const int32 InsertAtIndex, const TArray<int32>& SelectedBones, bool CalcNewLocalTransform, bool Validate)
{
	check(GeometryCollection);

	TManagedArray<int32>& Parents = GeometryCollection->Parent;
	int32 ParentIdx = InsertAtIndex < Parents.Num() ? Parents[InsertAtIndex] : INDEX_NONE;
	return ClusterBonesUnderNewNodeWithParent(GeometryCollection, ParentIdx, SelectedBones, CalcNewLocalTransform, Validate);
}

int32 FGeometryCollectionClusteringUtility::ClusterBonesUnderNewNodeWithParent(FGeometryCollection* GeometryCollection, const int32 ParentOfNewNode, const TArray<int32>& SelectedBones, bool CalcNewLocalTransform, bool Validate)
{
	check(GeometryCollection);


	TManagedArray<FTransform3f>& Transforms = GeometryCollection->Transform;
	TManagedArray<FString>& BoneNames = GeometryCollection->BoneName;
	TManagedArray<int32>& Parents = GeometryCollection->Parent;
	TManagedArray<TSet<int32>>& Children = GeometryCollection->Children;
	TManagedArray<int32>& SimType = GeometryCollection->SimulationType;

	// insert a new node between the selected bones and their shared parent
	int NewBoneIndex = GeometryCollection->AddElements(1, FGeometryCollection::TransformGroup);

	int32 OriginalParentIndex = ParentOfNewNode;
	Parents[NewBoneIndex] = OriginalParentIndex;
	Children[NewBoneIndex] = TSet<int32>(SelectedBones);
	SimType[NewBoneIndex] = FGeometryCollection::ESimulationTypes::FST_Clustered;

	Transforms[NewBoneIndex] = FTransform3f::Identity;

	// re-parent all the geometry nodes under the new shared bone
	GeometryCollectionAlgo::ParentTransforms(GeometryCollection, NewBoneIndex, SelectedBones);

	UpdateHierarchyLevelOfChildren(GeometryCollection, NewBoneIndex);

	// Parent Bone Fixup of Children - add the new node under the first bone selected
	// #todo: might want to add it to the one closest to the root in the hierarchy
	if (OriginalParentIndex != FGeometryCollection::Invalid)
	{
		Children[OriginalParentIndex].Add(NewBoneIndex);
	}

	// Update new cluster's bone name
	BoneNames[NewBoneIndex] = FString::Printf(TEXT("ClusterBone_%d"), NewBoneIndex);

	if (Validate)
	{
		ValidateResults(GeometryCollection);
	}

	return NewBoneIndex;
}

void FGeometryCollectionClusteringUtility::ClusterAllBonesUnderNewRoot(FGeometryCollection* GeometryCollection, FName RootName, bool bUpdateChildBoneNames)
{
	check(GeometryCollection);
	bool CalcNewLocalTransform = true;

	TManagedArray<FTransform3f>& Transforms = GeometryCollection->Transform;
	TManagedArray<FString>& BoneNames = GeometryCollection->BoneName;
	TManagedArray<int32>& Parents = GeometryCollection->Parent;
	TManagedArray<TSet<int32>>& Children = GeometryCollection->Children;
	TManagedArray<int32>& SimulationType = GeometryCollection->SimulationType;
	TManagedArray<FLinearColor>& BoneColors = GeometryCollection->BoneColor;

	TArray<int32> ChildBones;
	int32 NumElements = GeometryCollection->NumElements(FGeometryCollection::TransformGroup);
	for (int ChildIndex = 0; ChildIndex < NumElements; ChildIndex++)
	{
		if (Parents[ChildIndex] == FGeometryCollection::Invalid)
			ChildBones.Push(ChildIndex);
	}

	// insert a new Root node
	int RootNoneIndex = GeometryCollection->AddElements(1, FGeometryCollection::TransformGroup);

	if (GeometryCollection->HasAttribute("Level", FGeometryCollection::TransformGroup))
	{
		TManagedArray<int32>& Levels = GeometryCollection->ModifyAttribute<int32>("Level", FGeometryCollection::TransformGroup);

		// all bones shifted down one in hierarchy
		for (int ChildIndex = 0; ChildIndex < NumElements; ChildIndex++)
		{
			Levels[ChildIndex] += 1;
		}
		Levels[RootNoneIndex] = 0;
	}

	// New Bone Setup takes level/parent from the first of the Selected Bones
	if (RootName.IsNone())
	{
		BoneNames[RootNoneIndex] = "ClusterBone";
	}
	else
	{
		BoneNames[RootNoneIndex] = RootName.ToString();
	}
	Parents[RootNoneIndex] = FGeometryCollection::Invalid;
	Children[RootNoneIndex] = TSet<int32>(ChildBones);
	SimulationType[RootNoneIndex] = FGeometryCollection::ESimulationTypes::FST_Clustered;
	check(GeometryCollection->IsTransform(RootNoneIndex));

	if (GeometryCollection->HasAttribute("ExplodedVector", FGeometryCollection::TransformGroup) &&
		GeometryCollection->HasAttribute("ExplodedTransform", FGeometryCollection::TransformGroup) )
	{

		TManagedArray<FVector3f>& ExplodedVectors = GeometryCollection->ModifyAttribute<FVector3f>("ExplodedVector", FGeometryCollection::TransformGroup);
		TManagedArray<FTransform>& ExplodedTransforms = GeometryCollection->ModifyAttribute<FTransform>("ExplodedTransform", FGeometryCollection::TransformGroup);

		FVector3f SumOfOffsets(0, 0, 0);
		for (int32 ChildBoneIndex : ChildBones)
		{
			ExplodedVectors[ChildBoneIndex] = FVector3f(Transforms[ChildBoneIndex].GetLocation());
			ExplodedTransforms[ChildBoneIndex] = FTransform(Transforms[ChildBoneIndex]);
			SumOfOffsets += ExplodedVectors[ChildBoneIndex];
		}
		ExplodedTransforms[RootNoneIndex] = FTransform(Transforms[RootNoneIndex]);
		// This bones offset is the average of all the selected bones
		ExplodedVectors[RootNoneIndex] = SumOfOffsets / static_cast<float>(ChildBones.Num());
	}

	// Selected Bone Setup
	for (int32 ChildBoneIndex : ChildBones)
	{
		Parents[ChildBoneIndex] = RootNoneIndex;
	}

	Transforms[RootNoneIndex] = FTransform3f::Identity;

	if (bUpdateChildBoneNames)
	{
		RecursivelyUpdateChildBoneNames(RootNoneIndex, Children, BoneNames);
	}

	const FColor RandBoneColor(FMath::Rand() % 100 + 5, FMath::Rand() % 100 + 5, FMath::Rand() % 100 + 5, 255);
	BoneColors[RootNoneIndex] = FLinearColor(RandBoneColor);

	ValidateResults(GeometryCollection);
}


void FGeometryCollectionClusteringUtility::ClusterBonesUnderExistingRoot(FGeometryCollection* GeometryCollection, const TArray<int32>& SourceElements)
{
	check(GeometryCollection);
	bool CalcNewLocalTransform = true;

	TManagedArray<int32>& Levels = GeometryCollection->ModifyAttribute<int32>("Level", FGeometryCollection::TransformGroup);

	TManagedArray<FString>& BoneNames = GeometryCollection->BoneName;
	TManagedArray<int32>& Parents = GeometryCollection->Parent;
	TManagedArray<TSet<int32>>& Children = GeometryCollection->Children;
	TManagedArray<int32>& SimulationType = GeometryCollection->SimulationType;

	TArray<int32> RootBonesOut;
	GetRootBones(GeometryCollection, RootBonesOut);
	check(RootBonesOut.Num() == 1); // only expecting a single root node
	int32 RootBoneElement = RootBonesOut[0];
	check(Levels[RootBoneElement] == 0);
	check(Parents[RootBoneElement] == FGeometryCollection::Invalid);

	// re-parent all the geometry nodes under the root node
	GeometryCollectionAlgo::ParentTransforms(GeometryCollection, RootBoneElement, SourceElements);

	// update source levels and transforms in our custom attributes
	for (int32 Element : SourceElements)
	{
		if (Element != RootBoneElement)
		{
			Levels[Element] = 1;
		}
	}

	// delete all the redundant transform nodes that we no longer use
	TArray<int32> NodesToDelete;
	for (int Element = 0; Element < GeometryCollection->NumElements(FGeometryCollection::TransformGroup); Element++)
	{
		if (Element != RootBoneElement && GeometryCollection->IsTransform(Element))
		{
			NodesToDelete.Add(Element);
		}
	}

	if (NodesToDelete.Num() > 0)
	{
		NodesToDelete.Sort();
		FManagedArrayCollection::FProcessingParameters Params;
		Params.bDoValidation = false;
		GeometryCollection->RemoveElements(FGeometryCollection::TransformGroup, NodesToDelete, Params);
	}

	// the root bone index could have changed after the above RemoveElements
	RootBonesOut.Empty();
	GetRootBones(GeometryCollection, RootBonesOut);
	RootBoneElement = RootBonesOut[0];

	RecursivelyUpdateChildBoneNames(RootBoneElement, Children, BoneNames);

	ValidateResults(GeometryCollection);
}

void FGeometryCollectionClusteringUtility::ClusterBonesUnderExistingNode(FGeometryCollection* GeometryCollection, const TArray<int32>& SourceElements)
{
	int32 MergeNode = PickBestNodeToMergeTo(GeometryCollection, SourceElements);
	ClusterBonesUnderExistingNode(GeometryCollection, MergeNode, SourceElements);

}

void FGeometryCollectionClusteringUtility::ClusterBonesUnderExistingNode(FGeometryCollection* GeometryCollection, int32 MergeNode, const TArray<int32>& SourceElementsIn)
{
	check(GeometryCollection);
	bool CalcNewLocalTransform = true;

	TManagedArray<int32>& Parents = GeometryCollection->Parent;
	TManagedArray<TSet<int32>>& Children = GeometryCollection->Children;
	TManagedArray<FString>& BoneNames = GeometryCollection->BoneName;

	// These attributes are apparently deprecated?
	//TManagedArray<FTransform>& ExplodedTransforms = GeometryCollection->GetAttribute<FTransform>("ExplodedTransform", FGeometryCollection::TransformGroup);
	//TManagedArray<FVector3f>& ExplodedVectors = GeometryCollection->GetAttribute<FVector3f>("ExplodedVector", FGeometryCollection::TransformGroup);

	// remove Merge Node if it's in the list - happens due to the way selection works
	TArray<int32> SourceElements;
	for (int32 Element : SourceElementsIn)
	{
		if (Element != MergeNode)
		{
			SourceElements.Push(Element);
		}
	}

	if (MergeNode != FGeometryCollection::Invalid)
	{
		bool IllegalOperation = false;
		for (int32 SourceElement : SourceElements)
		{
			if (NodeExistsOnThisBranch(GeometryCollection, MergeNode, SourceElement))
			{
				IllegalOperation = true;
				break;
			}
		}

		if (!IllegalOperation)
		{
			TArray<int32> ParentsToUpdateNames;
			// determine original parents of moved nodes so we can update their children's names
			for (int32 SourceElement : SourceElementsIn)
			{
				int32 Parent = Parents[SourceElement];
				if (Parent != FGeometryCollection::Invalid)
				{
					ParentsToUpdateNames.AddUnique(Parent);
				}
			}

			//ResetSliderTransforms(ExplodedTransforms, Transforms);

			// re-parent all the geometry nodes under existing merge node
			GeometryCollectionAlgo::ParentTransforms(GeometryCollection, MergeNode, SourceElements);

			// update source levels and transforms in our custom attributes
			//for (int32 Element : SourceElements)
			//{
			//	ExplodedTransforms[Element] = Transforms[Element];
			//	ExplodedVectors[Element] = Transforms[Element].GetLocation();
			//}

			UpdateHierarchyLevelOfChildren(GeometryCollection, MergeNode);

			RecursivelyUpdateChildBoneNames(MergeNode, Children, BoneNames);

			for (int32 NodeIndex : ParentsToUpdateNames)
			{
				if (NodeIndex != INDEX_NONE)
				{
					RecursivelyUpdateChildBoneNames(NodeIndex, Children, BoneNames);
				}
			}
		}
	}

	// add common root node if multiple roots found
	if (FGeometryCollectionClusteringUtility::ContainsMultipleRootBones(GeometryCollection))
	{
		FGeometryCollectionClusteringUtility::ClusterAllBonesUnderNewRoot(GeometryCollection);
	}

	ValidateResults(GeometryCollection);
}

void FGeometryCollectionClusteringUtility::ClusterBonesByContext(FGeometryCollection* GeometryCollection, int32 MergeNode, const TArray<int32>& SourceElementsIn)
{
	if (GeometryCollection->IsTransform(MergeNode))
	{
		ClusterBonesUnderExistingNode(GeometryCollection, MergeNode, SourceElementsIn);
	}
	else
	{
		TArray<int32> SourceElements = SourceElementsIn;
		SourceElements.Push(MergeNode);
		ClusterBonesUnderNewNode(GeometryCollection, MergeNode, SourceElements, true);
	}
}

void FGeometryCollectionClusteringUtility::CollapseHierarchyOneLevel(FGeometryCollection* GeometryCollection, TArray<int32>& SourceElements)
{
	check(GeometryCollection);
	bool CalcNewLocalTransform = true;

	TManagedArray<int32>& Parents = GeometryCollection->Parent;
	TManagedArray<TSet<int32>>& Children = GeometryCollection->Children;
	TManagedArray<FString>& BoneNames = GeometryCollection->BoneName;
	TManagedArray<int32>& Levels = GeometryCollection->ModifyAttribute<int32>("Level", FGeometryCollection::TransformGroup);

	for (int32 SourceElement : SourceElements)
	{
		int32 DeletedNode = SourceElement;
		if (DeletedNode != FGeometryCollection::Invalid)
		{
			int32 NewParentElement = Parents[DeletedNode];

			if (NewParentElement != FGeometryCollection::Invalid)
			{
				for (int32 ChildElement : Children[DeletedNode])
				{
					Children[NewParentElement].Add(ChildElement);
					Levels[ChildElement] -= 1;
					Parents[ChildElement] = NewParentElement;
				}
				Children[DeletedNode].Empty();
			}
		}
	}

	SourceElements.Sort();
	GeometryCollection->RemoveElements(FGeometryCollection::TransformGroup, SourceElements);

	TArray<int32> Roots;
	GetRootBones(GeometryCollection, Roots);
	if (!Roots.IsEmpty())
	{
		RecursivelyUpdateChildBoneNames(Roots[0], Children, BoneNames);
	}

	ValidateResults(GeometryCollection);
}


bool FGeometryCollectionClusteringUtility::NodeExistsOnThisBranch(const FGeometryCollection* GeometryCollection, int32 TestNode, int32 TreeElement)
{
	const TManagedArray<TSet<int32>>& Children = GeometryCollection->Children;

	if (TestNode == TreeElement)
		return true;

	if (Children[TreeElement].Num() > 0)
	{
		for (int32 ChildIndex : Children[TreeElement])
		{
			if (NodeExistsOnThisBranch(GeometryCollection, TestNode, ChildIndex))
				return true;
		}
	}

	return false;

}

void FGeometryCollectionClusteringUtility::RenameBone(FGeometryCollection* GeometryCollection, int32 BoneIndex, const FString& NewName, bool UpdateChildren /* = true */)
{
	TManagedArray<FString>& BoneNames = GeometryCollection->BoneName;
	const TManagedArray<TSet<int32>>& Children = GeometryCollection->Children;

	BoneNames[BoneIndex] = NewName;

	if (UpdateChildren)
	{
		FGeometryCollectionClusteringUtility::RecursivelyUpdateChildBoneNames(BoneIndex, Children, BoneNames, true);
	}
}

int32 FGeometryCollectionClusteringUtility::PickBestNodeToMergeTo(const FManagedArrayCollection* Collection, const TArray<int32>& SourceElements)
{
	const Chaos::Facades::FCollectionHierarchyFacade HierarchyFacade(*Collection);
	if (!HierarchyFacade.IsValid() || !HierarchyFacade.HasLevelAttribute())
	{
		return -1;
	}

	// which of the source elements is the most significant, closest to the root that has children (is a cluster)
	int32 ElementClosestToRoot = -1;
	int32 LevelClosestToRoot = -1;

	for (int32 Element : SourceElements)
	{
		const TSet<int32>* Children = HierarchyFacade.FindChildren(Element);
		int32 Level = HierarchyFacade.GetInitialLevel(Element);
		if (Children && !Children->IsEmpty() && (Level < LevelClosestToRoot || LevelClosestToRoot == -1))
		{
			LevelClosestToRoot = Level;
			ElementClosestToRoot = Element;
		}
	}

	return ElementClosestToRoot;
}

void FGeometryCollectionClusteringUtility::ResetSliderTransforms(TManagedArray<FTransform>& ExplodedTransforms, TManagedArray<FTransform>& Transforms)
{
	for (int Element = 0; Element < Transforms.Num(); Element++)
	{
		Transforms[Element] = ExplodedTransforms[Element];
	}
}

bool FGeometryCollectionClusteringUtility::ContainsMultipleRootBones(FGeometryCollection* GeometryCollection)
{
	check(GeometryCollection);
	const TManagedArray<int32>& Parents = GeometryCollection->Parent;

	// never assume the root bone is always index 0 in the particle group
	int NumRootBones = 0;
	for (int i = 0; i < Parents.Num(); i++)
	{
		if (Parents[i] == FGeometryCollection::Invalid)
		{
			NumRootBones++;
			if (NumRootBones > 1)
			{
				return true;
			}
		}
	}
	return false;
}


void FGeometryCollectionClusteringUtility::GetRootBones(const FGeometryCollection* GeometryCollection, TArray<int32>& RootBonesOut)
{
	check(GeometryCollection);
	checkSlow(RootBonesOut.Num() == 0);
	const TManagedArray<int32>& Parents = GeometryCollection->Parent;

	// never assume the root bone is always index 0 in the particle group
	for (int i = 0; i < Parents.Num(); i++)
	{
		if (Parents[i] == FGeometryCollection::Invalid)
		{
			RootBonesOut.Add(i);
		}
	}
}

bool FGeometryCollectionClusteringUtility::IsARootBone(const FGeometryCollection* GeometryCollection, int32 InBone)
{
	check(GeometryCollection);
	const TManagedArray<int32>& Parents = GeometryCollection->Parent;

	return (Parents[InBone] == FGeometryCollection::Invalid);
}

void FGeometryCollectionClusteringUtility::GetClusteredBonesWithCommonParent(const FGeometryCollection* GeometryCollection, int32 SourceBone, TArray<int32>& BonesOut)
{
	check(GeometryCollection);
	const TManagedArray<int32>& Parents = GeometryCollection->Parent;
	const TManagedArray<int32>& SimulationType = GeometryCollection->SimulationType;

	// then see if this bone as any other bones clustered to it
	if (SimulationType[SourceBone] == FGeometryCollection::ESimulationTypes::FST_Clustered)
	{
		int32 SourceParent = Parents[SourceBone];

		for (int i = 0; i < Parents.Num(); i++)
		{
			if (SourceParent == Parents[i] && (SimulationType[i] == FGeometryCollection::ESimulationTypes::FST_Clustered))
				BonesOut.AddUnique(i);
		}
	}

}

void FGeometryCollectionClusteringUtility::GetChildBonesFromLevel(const FGeometryCollection* GeometryCollection, int32 SourceBone, int32 Level, TArray<int32>& BonesOut)
{
	check(GeometryCollection);
	if (!ensure(GeometryCollection->HasAttribute("Level", FGeometryCollection::TransformGroup)))
	{
		return;
	}
	const TManagedArray<int32>& Levels = GeometryCollection->GetAttribute<int32>("Level", FGeometryCollection::TransformGroup);
	const TManagedArray<int32>& Parents = GeometryCollection->Parent;
	const TManagedArray<TSet<int32>>& Children = GeometryCollection->Children;

	if (SourceBone >= 0)
	{
		int32 SourceParent = SourceBone;
		while (Levels[SourceParent] > Level)
		{
			if (Parents[SourceParent] == -1)
				break;

			SourceParent = Parents[SourceParent];
		}

		RecursiveAddAllChildren(Children, SourceParent, BonesOut);
	}

}

void FGeometryCollectionClusteringUtility::GetBonesToLevel(const FGeometryCollection* GeometryCollection, int32 Level, TArray<int32>& BonesOut, bool bOnlyClusteredOrRigid, bool bSkipFiltered)
{
	check(GeometryCollection);
	
	if (!ensure(GeometryCollection->HasAttribute("Level", FGeometryCollection::TransformGroup)))
	{
		return;
	}
	const TManagedArray<int32>& Levels = GeometryCollection->GetAttribute<int32>("Level", FGeometryCollection::TransformGroup);
	const TManagedArray<int32>& SimType = GeometryCollection->SimulationType;

	bool bAllLevels = Level == -1;

	int32 NumBones = GeometryCollection->NumElements(FGeometryCollection::TransformGroup);
	for (int32 BoneIdx = 0; BoneIdx < NumBones; BoneIdx++)
	{
		bool bIsRigid = SimType[BoneIdx] == FGeometryCollection::ESimulationTypes::FST_Rigid;
		bool bIsClustered = SimType[BoneIdx] == FGeometryCollection::ESimulationTypes::FST_Clustered;
		if (
			// (if skipping embedded) sim type is clustered or rigid 
			(!bOnlyClusteredOrRigid || bIsClustered || bIsRigid)
			&&
			// (if skipping nodes the outliner has filtered) sim type is clustered or level is an exact match or level has an exact-match child
			(bAllLevels || !bSkipFiltered || bIsClustered || Levels[BoneIdx] == Level || (GeometryCollection->Children[BoneIdx].Num() > 0 && Levels[BoneIdx] + 1 == Level))
			&&
			// level is at or before the target
			(bAllLevels || Levels[BoneIdx] <= Level)
			)
		{
			BonesOut.Add(BoneIdx);
		}
	}
}

void FGeometryCollectionClusteringUtility::GetChildBonesAtLevel(const FGeometryCollection* GeometryCollection, int32 SourceBone, int32 Level, TArray<int32>& BonesOut)
{
	check(GeometryCollection);

	if (Level == -1)
	{
		GetLeafBones(GeometryCollection, SourceBone, false, BonesOut);
	}
	else
	{
		if (!ensure(GeometryCollection->HasAttribute("Level", FGeometryCollection::TransformGroup)))
		{
			return;
		}
		const TManagedArray<int32>& Levels = GeometryCollection->GetAttribute<int32>("Level", FGeometryCollection::TransformGroup);
		const TManagedArray<TSet<int32>>& Children = GeometryCollection->Children;
		if (Levels[SourceBone] == Level)
		{
			BonesOut.Push(SourceBone);
		}
		else
		{
			for (int32 Child : Children[SourceBone])
			{
				GetChildBonesAtLevel(GeometryCollection, Child, Level, BonesOut);
			}
		}
	}
}

void FGeometryCollectionClusteringUtility::RecursiveAddAllChildren(const TManagedArray<TSet<int32>>& Children, int32 SourceBone, TArray<int32>& BonesOut)
{
	if (BonesOut.Contains(SourceBone))
	{
		// likely a recursive loop, we need to break the cycle as soon as possible 
		return;
	}
	BonesOut.Add(SourceBone);
	for (int32 Child : Children[SourceBone])
	{
		RecursiveAddAllChildren(Children, Child, BonesOut);
	}

}

int32 FGeometryCollectionClusteringUtility::GetParentOfBoneAtSpecifiedLevel(const FGeometryCollection* GeometryCollection, int32 SourceBone, int32 Level, bool bSkipFiltered)
{
	check(GeometryCollection);
	const TManagedArray<int32>& Parents = GeometryCollection->Parent;
	if (!ensure(GeometryCollection->HasAttribute("Level", FGeometryCollection::TransformGroup)))
	{
		return -1;
	}
	const TManagedArray<int32>& Levels = GeometryCollection->GetAttribute<int32>("Level", FGeometryCollection::TransformGroup);
	const TManagedArray<int32>& SimTypes = GeometryCollection->SimulationType;

	if (SourceBone >= 0 && SourceBone < Parents.Num())
	{
		int32 SourceParent = SourceBone;
		while (Levels[SourceParent] > Level || 
			// go to parents of bones that will be filtered by the outliner (i.e., rigid/embedded at the wrong level)
			(bSkipFiltered && 
				Levels[SourceParent] != Level && 
				GeometryCollection->SimulationType[SourceParent] != FGeometryCollection::ESimulationTypes::FST_Clustered &&
				(GeometryCollection->Children[SourceParent].Num() == 0 || Levels[SourceParent] + 1 != Level)
			))
		{
			if (Parents[SourceParent] == -1)
			{
				break;
			}

			SourceParent = Parents[SourceParent];
		}

		return SourceParent;
	}

	return FGeometryCollection::Invalid;
}

void FGeometryCollectionClusteringUtility::RecursivelyUpdateChildBoneNames(int32 BoneIndex, const TManagedArray<TSet<int32>>& Children, TManagedArray<FString>& BoneNames, bool OverrideBoneNames /*= false*/)
{
	if (!ensure(BoneIndex > -1 && BoneIndex < Children.Num()))
	{
		return;
	}

	if (Children[BoneIndex].Num() > 0)
	{
		const FString& ParentName = BoneNames[BoneIndex];
		int DisplayIndex = 1;
		for (int32 ChildIndex : Children[BoneIndex])
		{
			FString NewName;
			int32 FoundIndex = 0;
			FString ChunkNumberStr( FString::FromInt(DisplayIndex++) );

			// enable this if we don't want to override the child names with parent names
			bool HasExistingName = BoneNames[ChildIndex].FindChar('_', FoundIndex);

			if (!OverrideBoneNames && HasExistingName && FoundIndex > 0)
			{
				FString CurrentName = BoneNames[ChildIndex].Left(FoundIndex);

				int32 FoundNumberIndex = 0;
				bool ParentHasNumbers = ParentName.FindChar('_', FoundNumberIndex);
				if (ParentHasNumbers && FoundNumberIndex > 0)
				{
					FString ParentNumbers = ParentName.Right(ParentName.Len() - FoundNumberIndex);
					NewName = CurrentName + ParentNumbers + "_" + ChunkNumberStr;
				}
				else
				{
					NewName = CurrentName + "_" + ChunkNumberStr;
				}
			}
			else
			{
				NewName = ParentName + "_" + ChunkNumberStr;
			}
			BoneNames[ChildIndex] = NewName;
			RecursivelyUpdateChildBoneNames(ChildIndex, Children, BoneNames, OverrideBoneNames);
		}
	}
}

void FGeometryCollectionClusteringUtility::UpdateHierarchyLevelOfChildren(FGeometryCollection* GeometryCollection, int32 ParentElement)
{
	if (!GeometryCollection->HasAttribute("Level", FGeometryCollection::TransformGroup))
	{
		GeometryCollection->AddAttribute<int32>("Level", FGeometryCollection::TransformGroup);
	}
	TManagedArray<int32>& Levels = GeometryCollection->ModifyAttribute<int32>("Level", FGeometryCollection::TransformGroup);
	const TManagedArray<TSet<int32>>& Children = GeometryCollection->Children;
	check(ParentElement < Levels.Num());
	check(ParentElement < Children.Num());

	if (ParentElement != INDEX_NONE)
	{
		RecursivelyUpdateHierarchyLevelOfChildren(Levels, Children, ParentElement);
	}
	else
	{
		TArray<int32> RootBonesOut;
		GetRootBones(GeometryCollection, RootBonesOut);
		for (int32 RootBone : RootBonesOut)
		{
			RecursivelyUpdateHierarchyLevelOfChildren(Levels, Children, RootBone);
		}
	}
}

void FGeometryCollectionClusteringUtility::UpdateHierarchyLevelOfChildren(FManagedArrayCollection& InCollection, int32 ParentElement)
{
	Chaos::Facades::FCollectionHierarchyFacade HierarchyFacade(InCollection);

	HierarchyFacade.GenerateLevelAttribute();
}

void FGeometryCollectionClusteringUtility::RecursivelyUpdateHierarchyLevelOfChildren(TManagedArray<int32>& Levels, const TManagedArray<TSet<int32>>& Children, int32 ParentElement)
{
	check(ParentElement < Levels.Num());
	check(ParentElement < Children.Num());

	for (int32 Element : Children[ParentElement])
	{
		Levels[Element] = Levels[ParentElement] + 1;
		RecursivelyUpdateHierarchyLevelOfChildren(Levels, Children, Element);
	}
}

void FGeometryCollectionClusteringUtility::CollapseLevelHierarchy(int8 Level, FGeometryCollection* GeometryCollection)
{
	check(GeometryCollection);

	if (!GeometryCollection->HasAttribute("Level", FGeometryCollection::TransformGroup))
	{
		UpdateHierarchyLevelOfChildren(GeometryCollection, -1);
	}
	const TManagedArray<int32>& Levels = GeometryCollection->GetAttribute<int32>("Level", FGeometryCollection::TransformGroup);

	TArray<int32> Elements;

	if (Level == -1) // AllLevels
	{

		for (int Element = 0; Element < GeometryCollection->NumElements(FGeometryCollection::TransformAttribute); Element++)
		{
			if (GeometryCollection->IsGeometry(Element))
			{
				Elements.Add(Element);
			}
		}

		if (Elements.Num() > 0)
		{
			ClusterBonesUnderExistingRoot(GeometryCollection, Elements);
		}
	}
	else
	{
		for (int Element = 0; Element < GeometryCollection->NumElements(FGeometryCollection::TransformAttribute); Element++)
		{
			// if matches selected level then re-parent this node to the root
			if (Levels[Element] == Level)
			{
				Elements.Add(Element);
			}
		}
		if (Elements.Num() > 0)
		{
			CollapseHierarchyOneLevel(GeometryCollection, Elements);
		}
	}

}

void FGeometryCollectionClusteringUtility::CollapseSelectedHierarchy(int8 Level, const TArray<int32>& SelectedBones, FGeometryCollection* GeometryCollection)
{
	check(GeometryCollection);
	if (!GeometryCollection->HasAttribute("Level", FGeometryCollection::TransformGroup))
	{
		UpdateHierarchyLevelOfChildren(GeometryCollection, -1);
	}
	const TManagedArray<int32>& Levels = GeometryCollection->GetAttribute<int32>("Level", FGeometryCollection::TransformGroup);
	const TManagedArray<TSet<int32>>& Children = GeometryCollection->Children;

	// can't collapse root node away and doesn't make sense to operate when AllLevels selected
	if (Level > 0)
	{
		TArray<int32> Elements;
		for (int32 Element = 0; Element < SelectedBones.Num(); Element++)
		{
			int32 Index = SelectedBones[Element];

			// if matches selected level then re-parent this node to the root if it's not a leaf node
			if (Levels[Index] == Level && Children[Index].Num() > 0)
			{
				Elements.Add(SelectedBones[Element]);
			}
		}

		if (Elements.Num() > 0)
		{
			CollapseHierarchyOneLevel(GeometryCollection, Elements);
		}
	}

}

void FGeometryCollectionClusteringUtility::ValidateResults(FGeometryCollection* GeometryCollection)
{
	if (ChaosValidateResultsOfEditOperations)
	{
		const TManagedArray<int32>& Parents = GeometryCollection->Parent;
		const TManagedArray<TSet<int32>>& Children = GeometryCollection->Children;
		const TManagedArray<FString>& BoneNames = GeometryCollection->BoneName;

		// there should only ever be one root node
		int NumRootNodes = 0;
		for (int i = 0; i < Parents.Num(); i++)
		{
			if (Parents[i] == FGeometryCollection::Invalid)
			{
				NumRootNodes++;
			}
		}
		check(NumRootNodes == 1);

		ensure(GeometryCollection->HasContiguousFaces());
		ensure(GeometryCollection->HasContiguousVertices());
	}
}

void FGeometryCollectionClusteringUtility::GetLeafBones(const FManagedArrayCollection* Collection, int BoneIndex, bool bOnlyRigids, TArray<int32>& LeafBonesOut)
{
	if (!ensure(BoneIndex >= 0 && Collection != nullptr))
	{
		return;
	}

	const TManagedArrayAccessor<TSet<int32>> ChildrenAttribute(*Collection, FGeometryCollection::ChildrenAttribute, FGeometryCollection::TransformGroup);
	const TManagedArrayAccessor<int32> SimulationTypeAttribute(*Collection , FGeometryCollection::SimulationTypeAttribute, FGeometryCollection::TransformGroup);

	if (ChildrenAttribute.IsValid() && SimulationTypeAttribute.IsValid())
	{
		const TManagedArray<TSet<int32>>& Children = ChildrenAttribute.Get();
		const TManagedArray<int32>& SimulationType = SimulationTypeAttribute.Get();

		if (!bOnlyRigids && Children[BoneIndex].Num() == 0)
		{
			LeafBonesOut.Push(BoneIndex);
		}
		else if (bOnlyRigids && SimulationType[BoneIndex] == FGeometryCollection::ESimulationTypes::FST_Rigid)
		{
			LeafBonesOut.Push(BoneIndex);
		}
		else if (Children[BoneIndex].Num() > 0)
		{
			for (int32 ChildElement : Children[BoneIndex])
			{
				GetLeafBones(Collection, ChildElement, bOnlyRigids, LeafBonesOut);
			}
		}
	}
}

void FGeometryCollectionClusteringUtility::MoveUpOneHierarchyLevel(FGeometryCollection* GeometryCollection, const TArray<int32>& SelectedBones)
{
	check(GeometryCollection);

	TManagedArray<int32>& Parents = GeometryCollection->Parent;
	TManagedArray<TSet<int32>>& Children = GeometryCollection->Children;
	TManagedArray<FString>& BoneNames = GeometryCollection->BoneName;

	for (int32 BoneIndex : SelectedBones)
	{
		int32 Parent = Parents[BoneIndex];
		if (Parents[BoneIndex] != FGeometryCollection::Invalid)
		{
			int32 ParentsParent = Parents[Parent];
			if (ParentsParent != FGeometryCollection::Invalid)
			{
				TArray<int32> InBones;
				InBones.Push(BoneIndex);
				GeometryCollectionAlgo::ParentTransforms(GeometryCollection, ParentsParent, InBones);
				UpdateHierarchyLevelOfChildren(GeometryCollection, ParentsParent);
				RecursivelyUpdateChildBoneNames(ParentsParent, Children, BoneNames);
			}
		}
	}
	
	ValidateResults(GeometryCollection);
}


int32 FGeometryCollectionClusteringUtility::FindLowestCommonAncestor(const FManagedArrayCollection* Collection, const TArray<int32>& SelectedBones)
{
	const int32 SelectionCount = SelectedBones.Num();
	if (SelectionCount == 0)
	{
		return INDEX_NONE;
	}

	int32 LCA = SelectedBones[0];
	for (int32 Index = 1; Index < SelectionCount; ++Index)
	{
		if (LCA == INDEX_NONE)
		{
			return INDEX_NONE;
		}
		LCA = FindLowestCommonAncestor(Collection, LCA, SelectedBones[Index]);
	}
	return LCA;
}

int32 FGeometryCollectionClusteringUtility::FindLowestCommonAncestor(const FManagedArrayCollection* Collection, int32 N0, int32 N1)
{
	const TManagedArray<int32>* Parent = Collection->FindAttribute<int32>("Parent", FGeometryCollection::TransformGroup);
	if (!Parent)
	{
		return INDEX_NONE;
	}

	// Record the path to root from the first 
	TArray<int32> PathToRoot0;
	PathToRoot0.Add(N0);
	while (PathToRoot0.Last() != INDEX_NONE)
	{
		PathToRoot0.Add((*Parent)[PathToRoot0.Last()]);
	}

	// Traverse from the second node to root and return the first node found that is in the first path.
	int32 LCA = N1;
	while (LCA != INDEX_NONE)
	{
		if (PathToRoot0.Contains(LCA))
		{
			return LCA;
		}
		LCA = (*Parent)[LCA];
	}

	// No common ancestor
	return INDEX_NONE;
}

bool FGeometryCollectionClusteringUtility::RemoveClustersOfOnlyOneChild(FGeometryCollection* GeometryCollection)
{
	check(GeometryCollection);

	bool bRemovedAny = false;

	TArray<int32> DeletionList;
	do
	{
		DeletionList.Reset();

		for (int32 Idx = 0, Num = GeometryCollection->Transform.Num(); Idx < Num; ++Idx)
		{
			int32 ParentIdx = GeometryCollection->Parent[Idx];
			if (ParentIdx != INDEX_NONE && GeometryCollection->IsClustered(Idx))
			{
				if (GeometryCollection->Children[Idx].Num() == 1)
				{
					DeletionList.Add(Idx);
					GeometryCollectionAlgo::ParentTransforms(GeometryCollection, ParentIdx, GeometryCollection->Children[Idx].Array());
					UpdateHierarchyLevelOfChildren(GeometryCollection, ParentIdx);
					RecursivelyUpdateChildBoneNames(ParentIdx, GeometryCollection->Children, GeometryCollection->BoneName);
				}
			}
		}

		if (DeletionList.Num())
		{
			// Note: List is ordered by construction, so do not need to Sort()
			FManagedArrayCollection::FProcessingParameters Params;
			Params.bDoValidation = false; // for perf reasons
			GeometryCollection->RemoveElements(FGeometryCollection::TransformGroup, DeletionList, Params);
			bRemovedAny = true;
		}

		// Need to repeat until an iteration doesn't remove any nodes, to fully collapse any chains of single-child clusters
	} while (DeletionList.Num());

	return bRemovedAny;
}

bool FGeometryCollectionClusteringUtility::RemoveDanglingClusters(FGeometryCollection* GeometryCollection)
{
	check(GeometryCollection);

	const TManagedArray<int32>& SimulationType = GeometryCollection->SimulationType;

	const int32 TransformCount = GeometryCollection->Transform.Num();
	TArray<int32> DeletionList;
	for (int32 Idx = 0; Idx < TransformCount; ++Idx)
	{
		if(GeometryCollection->IsClustered(Idx))
		{
			TArray<int32> LeafBones;
			GetLeafBones(GeometryCollection, Idx, true, LeafBones);
			if (LeafBones.Num() == 0)
			{
				DeletionList.Add(Idx);
			}
		}
	}

	if (DeletionList.Num())
	{
		// Note: List is ordered by construction, so do not need to Sort()
		FManagedArrayCollection::FProcessingParameters Params;
		Params.bDoValidation = false; // for perf reasons
		GeometryCollection->RemoveElements(FGeometryCollection::TransformGroup, DeletionList, Params);
		return true;
	}

	return false;
}



