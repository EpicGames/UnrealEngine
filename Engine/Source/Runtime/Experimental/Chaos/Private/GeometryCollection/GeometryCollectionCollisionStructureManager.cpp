// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionCollisionStructureManager.h"
#include "ChaosLog.h"
#include "Chaos/Box.h"
#include "Chaos/ErrorReporter.h"
#include "Chaos/ImplicitObjectUnion.h"
#include "Chaos/Levelset.h"
#include "Chaos/Particles.h"
#include "Chaos/Sphere.h"
#include "Chaos/Capsule.h"
#include "Chaos/Vector.h"
#include "HAL/IConsoleManager.h"
#include "Chaos/TriangleMesh.h"
#include "Chaos/PBDRigidClustering.h"
#include "Chaos/PBDRigidClusteringCollisionParticleAlgo.h"

DEFINE_LOG_CATEGORY_STATIC(GCS_Log, NoLogging, All);

FCollisionStructureManager::FCollisionStructureManager()
{
}

int32 bCollisionParticlesUseImplicitCulling = false;
FAutoConsoleVariableRef CVarCollisionParticlesUseImplicitCulling(TEXT("p.CollisionParticlesUseImplicitCulling"), bCollisionParticlesUseImplicitCulling, TEXT("Use the implicit to cull interior vertices."));

int32 CollisionParticlesSpatialDivision = 10;
FAutoConsoleVariableRef CVarCollisionParticlesSpatialDivision(TEXT("p.CollisionParticlesSpatialDivision"), CollisionParticlesSpatialDivision, TEXT("Spatial bucketing to cull collision particles."));

int32 CollisionParticlesMin = 10;
FAutoConsoleVariableRef CVarCollisionParticlesMin(TEXT("p.CollisionParticlesMin"), CollisionParticlesMin, TEXT("Minimum number of particles after simplicial pruning (assuming it started with more)"));

int32 CollisionParticlesMax = 2000;
FAutoConsoleVariableRef CVarCollisionParticlesMax(TEXT("p.CollisionParticlesMax"), CollisionParticlesMax, TEXT("Maximum number of particles after simplicial pruning"));

FCollisionStructureManager::FSimplicial*
FCollisionStructureManager::NewSimplicial(
	const Chaos::FParticles& Vertices,
	const Chaos::FTriangleMesh& TriMesh,
	const Chaos::FImplicitObject* Implicit,
	const int32 CollisionParticlesMaxInput)
{
	FCollisionStructureManager::FSimplicial * Simplicial = new FCollisionStructureManager::FSimplicial();
	if (Implicit || Vertices.Size())
	{
		Chaos::FReal Extent = 0;
		int32 LSVCounter = 0;
		TArray<int32> IndicesArray;
		TriMesh.GetVertexSetAsArray(IndicesArray);
		TArray<Chaos::FVec3> OutsideVertices;

		bool bFullCopy = true;
		int32 LocalCollisionParticlesMax = CollisionParticlesMaxInput > 0 ? FMath::Min(CollisionParticlesMaxInput, CollisionParticlesMax) : CollisionParticlesMax;
		if (bCollisionParticlesUseImplicitCulling!=0 && Implicit && IndicesArray.Num()>LocalCollisionParticlesMax)
		{
			Extent = Implicit->HasBoundingBox() ? Implicit->BoundingBox().Extents().Size() : 1.f;

			Chaos::FReal Threshold = Extent * 0.01f;

			//
			//  Remove particles inside the levelset. (I think this is useless) 
			//
			OutsideVertices.AddUninitialized(IndicesArray.Num());
			for (int32 Idx : IndicesArray)
			{
				const Chaos::FVec3& SamplePoint = Vertices.GetX(Idx);
				if (Implicit->SignedDistance(SamplePoint) > Threshold)
				{
					OutsideVertices[LSVCounter] = SamplePoint;
					LSVCounter++;
				}
			}
			OutsideVertices.SetNum(LSVCounter);

			if (OutsideVertices.Num() > LocalCollisionParticlesMax)
				bFullCopy = false;
		}
		
		if(bFullCopy)
		{
			FBox Bounds(ForceInitToZero);
			OutsideVertices.AddUninitialized(IndicesArray.Num());
			for (int32 Idx=0;Idx<IndicesArray.Num();Idx++)
			{
				Bounds += FVector(Vertices.GetX(IndicesArray[Idx]));
				OutsideVertices[Idx] = Vertices.GetX(IndicesArray[Idx]);
			}
			Extent = Bounds.GetExtent().Size();
		}

		//
		// Clean the particles based on distance
		//

		Chaos::FReal SnapThreshold = Extent / Chaos::FReal(CollisionParticlesSpatialDivision);
		OutsideVertices = Chaos::CleanCollisionParticles(OutsideVertices, SnapThreshold);
		int32 NumParticles = (OutsideVertices.Num() > LocalCollisionParticlesMax) ? LocalCollisionParticlesMax : OutsideVertices.Num();

		if (NumParticles)
		{
			int32 VertexCounter = 0;
			Simplicial->AddParticles(NumParticles);
			for (int32 i = 0; i<NumParticles;i++)
			{
				if (!OutsideVertices[i].ContainsNaN())
				{
					Simplicial->SetX(i, OutsideVertices[i]);
					VertexCounter++;
				}
			}
			Simplicial->Resize(VertexCounter);
		}

		if(!Simplicial->Size())
		{
			Simplicial->AddParticles(1);
			Simplicial->SetX(0, Chaos::FVec3(0));
		}

		Simplicial->UpdateAccelerationStructures();

		UE_LOGF(LogChaos, Log, "NewSimplicial: InitialSize: %d, ImplicitExterior: %d, FullCopy: %d, FinalSize: %d", IndicesArray.Num(), LSVCounter, (int32)bFullCopy, NumParticles);
		return Simplicial;
	}
	UE_LOGF(LogChaos, Log, "NewSimplicial::Empty");
	return Simplicial;
}

FCollisionStructureManager::FSimplicial*
FCollisionStructureManager::NewSimplicial(
	const Chaos::FParticles& AllParticles,
	const TManagedArray<int32>& BoneMap,
	const ECollisionTypeEnum CollisionType,
	Chaos::FTriangleMesh& TriMesh,
	const float CollisionParticlesFraction)
{
	const bool bEnableCollisionParticles = (CollisionType == ECollisionTypeEnum::Chaos_Surface_Volumetric);
	if (bEnableCollisionParticles || true)
	{
		// @todo : Clean collision particles need to operate on the collision mask from the DynamicCollection,
		//         then transfer only the good collision particles during the initialization. `
		FCollisionStructureManager::FSimplicial * Simplicial = new FCollisionStructureManager::FSimplicial();
		const TArrayView<const Chaos::FVec3> ArrayView(&AllParticles.GetX(0), AllParticles.Size());
		const TArray<Chaos::FVec3>& Result = Chaos::CleanCollisionParticles(TriMesh, ArrayView, CollisionParticlesFraction);

		if (Result.Num())
		{
			int32 VertexCounter = 0;
			Simplicial->AddParticles(Result.Num());
			for (int Index = Result.Num() - 1; 0 <= Index; Index--)
			{
				if (!Result[Index].ContainsNaN())
				{
					Simplicial->SetX(Index, Result[Index]);
					VertexCounter++;
				}
			}
			Simplicial->Resize(VertexCounter);
		}

		if (!Simplicial->Size())
		{
			Simplicial->AddParticles(1);
			Simplicial->SetX(0, Chaos::FVec3(0));
		}

		Simplicial->UpdateAccelerationStructures();

		return Simplicial;
	}
	return new FCollisionStructureManager::FSimplicial();
}

void FCollisionStructureManager::UpdateImplicitFlags(
	FImplicit* Implicit, 
	const ECollisionTypeEnum CollisionType)
{
	if (ensure(Implicit))
	{
		Implicit->SetCollisionType(CollisionType == ECollisionTypeEnum::Chaos_Surface_Volumetric ? Chaos::ImplicitObjectType::LevelSet : Implicit->GetType());
		if (Implicit && (CollisionType == ECollisionTypeEnum::Chaos_Surface_Volumetric) && Implicit->GetType() == Chaos::ImplicitObjectType::LevelSet)
		{
			Implicit->SetDoCollide(false);
			Implicit->SetConvex(false);
		}
	}
}

Chaos::FImplicitObjectRef
FCollisionStructureManager::NewImplicit(
	Chaos::FErrorReporter ErrorReporter,
	const Chaos::FParticles& MeshParticles,
	const Chaos::FTriangleMesh& TriMesh,
	const FBox& CollisionBounds,
	const Chaos::FReal Radius,
	const int32 MinRes,
	const int32 MaxRes,
	const float CollisionObjectReduction,
	const ECollisionTypeEnum CollisionType,
	const EImplicitTypeEnum ImplicitType)
{
	if (ImplicitType == EImplicitTypeEnum::Chaos_Implicit_Box)
	{
		return NewImplicitBox(CollisionBounds, CollisionObjectReduction, CollisionType);
	}
	else if (ImplicitType == EImplicitTypeEnum::Chaos_Implicit_Sphere)
	{
		return NewImplicitSphere(Radius, CollisionObjectReduction, CollisionType);
	}
	else if (ImplicitType == EImplicitTypeEnum::Chaos_Implicit_LevelSet)
	{
		return NewImplicitLevelset(ErrorReporter, MeshParticles, TriMesh, CollisionBounds, MinRes, MaxRes, CollisionObjectReduction, CollisionType);
	}
	
	return nullptr;
}

Chaos::FImplicitObjectRef
FCollisionStructureManager::NewImplicitBox(
	const FBox& CollisionBounds,
	const float CollisionObjectReduction,
	const ECollisionTypeEnum CollisionType)
{
	const FVector HalfExtents = CollisionBounds.GetExtent() * (1 - CollisionObjectReduction / 100.f);
	const FVector Center = CollisionBounds.GetCenter();

	// Margin settings are in UPhysicsSettingsCore which we can't access here
	// @todo(chaos): pass margin settings into the collision manager?
	float CollisionMarginFraction = 0.1f;// FMath::Max(0.0f, UPhysicsSettingsCore::Get()->SolverOptions.CollisionMarginFraction);
	float CollisionMarginMax = 10.0f;// FMath::Max(0.0f, UPhysicsSettingsCore::Get()->SolverOptions.CollisionMarginMax);
	const Chaos::FReal Margin = FMath::Min(CollisionMarginFraction * 0.5f * HalfExtents.GetMin(), CollisionMarginMax);

	Chaos::FImplicitObjectRef Implicit = new Chaos::TBox<Chaos::FReal, 3>(Center - HalfExtents, Center + HalfExtents, Margin);
	UpdateImplicitFlags(Implicit, CollisionType);
	return Implicit;
}

Chaos::FImplicitObjectRef
FCollisionStructureManager::NewImplicitSphere(
	const Chaos::FReal Radius,
	const float CollisionObjectReduction,
	const ECollisionTypeEnum CollisionType)
{
	Chaos::FImplicitObjectRef Implicit = new Chaos::FSphere(Chaos::FVec3(0), Radius * (1 - CollisionObjectReduction / 100.f));
	UpdateImplicitFlags(Implicit, CollisionType);
	return Implicit;
}

Chaos::FImplicitObjectRef
FCollisionStructureManager::NewImplicitConvex(
	const TArray<int32>& ConvexIndices,
	const TManagedArray<Chaos::FConvexPtr>* ConvexGeometry,
	const ECollisionTypeEnum CollisionType,
	const FTransform& MassTransform,
	const Chaos::FReal CollisionMarginFraction,
	const float CollisionObjectReduction)
{
	using FConvexVec3 = Chaos::FConvex::FVec3Type;

	if (ConvexIndices.Num())
	{
		TArray<Chaos::FImplicitObjectRef> Implicits;
		for (auto& Index : ConvexIndices)
		{
			if((*ConvexGeometry)[Index])
			{
				TArray<FConvexVec3> ConvexVertices = (*ConvexGeometry)[Index]->GetVertices();
				FConvexVec3 COM = MassTransform.InverseTransformPosition((*ConvexGeometry)[Index]->GetCenterOfMass());
				FConvexVec3::FReal ScaleFactor = 1 - CollisionObjectReduction / 100.f;
				for (int32 Idx = 0; Idx < ConvexVertices.Num(); Idx++)
				{
					ConvexVertices[Idx] = ((FConvexVec3)MassTransform.InverseTransformPosition(FVector(ConvexVertices[Idx])) - COM) * ScaleFactor + COM;
				}

				Chaos::FReal Margin = (Chaos::FReal)(*ConvexGeometry)[Index]->BoundingBox().Extents().Min() * CollisionMarginFraction;
				Chaos::FConvex* MarginConvex = new Chaos::FConvex(ConvexVertices, Margin);
				if (MarginConvex->NumVertices() > 0)
				{
					Chaos::FImplicitObject* Implicit = MarginConvex;
					UpdateImplicitFlags(Implicit, CollisionType);
					Implicits.Add(Implicit);
				}
				else
				{
					delete MarginConvex;
				}
			}
		}

		if (Implicits.Num() == 0)
		{
			return nullptr;
		}
		else if (Implicits.Num() == 1)
		{
			return Implicits[0];
		}
		else
		{
			TArray<Chaos::FImplicitObjectPtr> ImplicitsPtrs;
			for(Chaos::FImplicitObjectRef ImplicitRef : Implicits)
			{
				Chaos::FImplicitObjectPtr ImplicitPtr(ImplicitRef);
				ImplicitsPtrs.Add(ImplicitPtr);
			}
			return new Chaos::FImplicitObjectUnion(MoveTemp(ImplicitsPtrs));
		}
	}
	return nullptr;
}

Chaos::FImplicitObjectRef
FCollisionStructureManager::NewImplicitCapsule(
	const Chaos::FReal Radius,
	const Chaos::FReal Length,
	const float CollisionObjectReduction,
	const ECollisionTypeEnum CollisionType)
{
	if (Length < UE_SMALL_NUMBER)
	{
		// make a more optimized shape : sphere
		return FCollisionStructureManager::NewImplicitSphere(Radius, CollisionObjectReduction, CollisionType);
	}
	
	const Chaos::FReal HalfLength = (Chaos::FReal)Length * (Chaos::FReal)0.5;
	Chaos::FImplicitObjectRef Implicit = new Chaos::FCapsule(Chaos::FVec3(0, 0, -HalfLength), Chaos::FVec3(0, 0, +HalfLength), Radius * (1 - CollisionObjectReduction / 100.f));
	UpdateImplicitFlags(Implicit, CollisionType);
	return Implicit;
}

Chaos::FImplicitObjectRef
FCollisionStructureManager::NewImplicitCapsule(
	const FBox& CollisionBounds,
	const float CollisionObjectReduction,
	const ECollisionTypeEnum CollisionType)
{
	const FVector BBoxCenter = CollisionBounds.GetCenter(); 
	const FVector BBoxExtent = CollisionBounds.GetExtent(); // FBox's extents are 1/2 (Max - Min)
	const Chaos::FReal XExtent = FMath::Abs(BBoxExtent.X);
	const Chaos::FReal YExtent = FMath::Abs(BBoxExtent.Y);
	const Chaos::FReal ZExtent = FMath::Abs(BBoxExtent.Z);
	Chaos::FVec3 HalfLengthVector;

	Chaos::FReal Radius = 0;
	if (XExtent > YExtent && XExtent > ZExtent)
	{
		Radius = FMath::Min(YExtent, ZExtent);
		HalfLengthVector = Chaos::FVec3((XExtent - Radius), 0, 0);
	}
	else if (YExtent > XExtent && YExtent > ZExtent)
	{
		Radius = FMath::Min(XExtent, ZExtent);
		HalfLengthVector = Chaos::FVec3(0, (YExtent - Radius), 0);
	}
	else
	{
		Radius = FMath::Min(XExtent, YExtent);
		HalfLengthVector = Chaos::FVec3(0, 0, (ZExtent - Radius));
	}

	if (HalfLengthVector.Size() < UE_SMALL_NUMBER)
	{
		// make a more optimized shape : sphere
		return FCollisionStructureManager::NewImplicitSphere(Radius, CollisionObjectReduction, CollisionType);
	}

	Chaos::FVec3 X1 = BBoxCenter - HalfLengthVector;
	Chaos::FVec3 X2 = BBoxCenter + HalfLengthVector;

	Chaos::FImplicitObjectRef Implicit = new Chaos::FCapsule(X1, X2, Radius * (1 - CollisionObjectReduction / 100.f));
	UpdateImplicitFlags(Implicit, CollisionType);
	return Implicit;
}

Chaos::FImplicitObjectRef
FCollisionStructureManager::NewImplicitLevelset(
	Chaos::FErrorReporter ErrorReporter,
	const Chaos::FParticles& MeshParticles,
	const Chaos::FTriangleMesh& TriMesh,
	const FBox& CollisionBounds,
	const int32 MinRes,
	const int32 MaxRes,
	const float CollisionObjectReduction,
	const ECollisionTypeEnum CollisionType)
{
	FVector HalfExtents = CollisionBounds.GetExtent();
	if (HalfExtents.GetAbsMin() < UE_KINDA_SMALL_NUMBER)
	{
		return nullptr;
	}
	Chaos::FLevelSetRef LevelSet = NewLevelset(ErrorReporter, MeshParticles, TriMesh, CollisionBounds, MinRes, MaxRes, CollisionType);
	if (LevelSet)
	{
		const Chaos::FReal DomainVolume = LevelSet->BoundingBox().GetVolume();
		const Chaos::FReal FilledVolume = LevelSet->ApproximateNegativeMaterial();
		if (FilledVolume < DomainVolume * 0.05)
		{
			const Chaos::FVec3 Extent = LevelSet->BoundingBox().Extents();
			ErrorReporter.ReportLog(
				*FString::Printf(TEXT(
					"Level set is small or empty:\n"
					"    domain extent: (%g %g %g) volume: %g\n"
					"    estimated level set volume: %g\n"
					"    percentage filled: %g%%"),
					Extent[0], Extent[1], Extent[2], DomainVolume, FilledVolume, FilledVolume / DomainVolume * 100.0));
		}
		HalfExtents *= CollisionObjectReduction / 100.f;
		const Chaos::FReal MinExtent = FMath::Min(HalfExtents[0], FMath::Min(HalfExtents[1], HalfExtents[2]));
		if (MinExtent > 0)
		{
			LevelSet->Shrink(MinExtent);
		}
	}
	else
	{
		ErrorReporter.ReportError(TEXT("Level set rasterization failed."));
	}
	return LevelSet;
}

Chaos::FLevelSetRef
FCollisionStructureManager::NewLevelset(
	Chaos::FErrorReporter ErrorReporter,
	const Chaos::FParticles& MeshParticles,
	const Chaos::FTriangleMesh& TriMesh,
	const FBox& CollisionBounds,
	const int32 MinRes,
	const int32 MaxRes,
	const ECollisionTypeEnum CollisionType)
{
	if(TriMesh.GetNumElements() == 0)
	{
		// Empty tri-mesh, can't create a new level set
		return nullptr;
	}

	Chaos::TVector<int32, 3> Counts;
	const FVector Extents = CollisionBounds.GetExtent();
	if (Extents.X < Extents.Y && Extents.X < Extents.Z)
	{
		Counts.X = MinRes;
		Counts.Y = MinRes * static_cast<int32>(Extents.Y / Extents.X);
		Counts.Z = MinRes * static_cast<int32>(Extents.Z / Extents.X);
	}
	else if (Extents.Y < Extents.Z)
	{
		Counts.X = MinRes * static_cast<int32>(Extents.X / Extents.Y);
		Counts.Y = MinRes;
		Counts.Z = MinRes * static_cast<int32>(Extents.Z / Extents.Y);
	}
	else
	{
		Counts.X = MinRes * static_cast<int32>(Extents.X / Extents.Z);
		Counts.Y = MinRes * static_cast<int32>(Extents.Y / Extents.Z);
		Counts.Z = MinRes;
	}
	if (Counts.X > MaxRes)
	{
		Counts.X = MaxRes;
	}
	if (Counts.Y > MaxRes)
	{
		Counts.Y = MaxRes;
	}
	if (Counts.Z > MaxRes)
	{
		Counts.Z = MaxRes;
	}
	Chaos::TUniformGrid<Chaos::FReal, 3> Grid(CollisionBounds.Min, CollisionBounds.Max, Counts, 1);
	Chaos::FLevelSetRef Implicit = new Chaos::FLevelSet(ErrorReporter, Grid, MeshParticles, TriMesh);
	if (ErrorReporter.ContainsUnhandledError())
	{
		ErrorReporter.HandleLatestError();	//Allow future levelsets to attempt to cook
		delete Implicit;
		return nullptr;
	}
	UpdateImplicitFlags(Implicit, CollisionType);
	return Implicit;
}

FVector 
FCollisionStructureManager::CalculateUnitMassInertiaTensor(
	const FBox& Bounds,
	const Chaos::FReal Radius,
	const EImplicitTypeEnum ImplicitType
)
{	
	FVector Tensor(1);
	if (ImplicitType == EImplicitTypeEnum::Chaos_Implicit_Box)
	{
		const Chaos::FVec3 Size = Bounds.GetSize();
		const Chaos::FMatrix33 I = Chaos::FAABB3::GetInertiaTensor(1.0, Size);
		Tensor = FVector(I.M[0][0], I.M[1][1], I.M[2][2]);
	}
	else if (ImplicitType == EImplicitTypeEnum::Chaos_Implicit_Sphere)
	{
		Tensor = FVector(Chaos::FSphere::GetInertiaTensor(1.0, Radius, false).M[0][0]);
	}
	ensureMsgf(Tensor.X != 0.f && Tensor.Y != 0.f && Tensor.Z != 0.f, TEXT("Rigid bounds check failure."));
	return Tensor;
}

Chaos::FReal
FCollisionStructureManager::CalculateVolume(
	const FBox& Bounds,
	const Chaos::FReal Radius,
	const EImplicitTypeEnum ImplicitType
)
{
	Chaos::FReal Volume = (Chaos::FReal)1.;
	if (ImplicitType == EImplicitTypeEnum::Chaos_Implicit_Box)
	{
		Volume = Bounds.GetVolume();
	}
	else if (ImplicitType == EImplicitTypeEnum::Chaos_Implicit_Sphere)
	{
		Volume = Chaos::FSphere::GetVolume(Radius);
	}
	ensureMsgf(Volume != 0.f, TEXT("Rigid volume check failure."));
	return Volume;
}
