// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionConnectionGraphUtility.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/Facades/CollectionConnectionGraphFacade.h"
#include "Containers/Set.h"
#include "Voronoi/Voronoi.h"

void FGeometryCollectionConnectionGraphGenerator::UpdateConnectivityGraph(FGeometryCollection& Collection, int32 ClusterTransformIndex, EClusterUnionMethod ConnectionMethod)
{
	FConnectionGraph Graph = ComputeConnectivityGraph(Collection, ClusterTransformIndex, ConnectionMethod);
	CommitToCollection(Graph, Collection, ClusterTransformIndex);
}

FGeometryCollectionConnectionGraphGenerator::FConnectionGraph
FGeometryCollectionConnectionGraphGenerator::ComputeConnectivityGraph(const FGeometryCollection& Collection, int32 ClusterTransformIndex, EClusterUnionMethod ConnectionMethod)
{
	FConnectionGraph Graph;
	Graph.SetNum(Collection.Children[ClusterTransformIndex].Num());

	// Connectivity Graph
	//    Build a connectivity graph for the cluster. If the PointImplicit is specified
	//    and the ClusterIndex has collision particles then use the expensive connection
	//    method. Otherwise try the DelaunayTriangulation if not none.
	//
	const bool bHasCollisionParticles = false; //Parent->CollisionParticles();
	EClusterUnionMethod LocalConnectionMethod = ConnectionMethod;
	if (LocalConnectionMethod == EClusterUnionMethod::None ||
	   (LocalConnectionMethod == EClusterUnionMethod::PointImplicit && !bHasCollisionParticles))
	{
		LocalConnectionMethod = EClusterUnionMethod::MinimalSpanningSubsetDelaunayTriangulation; // default method
	}

	if (LocalConnectionMethod == EClusterUnionMethod::PointImplicit ||
		LocalConnectionMethod == EClusterUnionMethod::PointImplicitAugmentedWithMinimalDelaunay)
	{
		// todo(chaos) implement point implicit version 
		check(false)
		//UpdateConnectivityGraphUsingPointImplicit(Parent, Parameters);
	}

	if (LocalConnectionMethod == EClusterUnionMethod::PointImplicitAugmentedWithMinimalDelaunay)
	{
		// todo : fix later when we have proper implementation for this on the GT side 
		FixConnectivityGraphUsingDelaunayTriangulation(Graph, Collection, ClusterTransformIndex);
	}
	
	if (LocalConnectionMethod == EClusterUnionMethod::DelaunayTriangulation)
	{
		ComputeConnectivityGraphUsingDelaunayTriangulation(Graph, Collection, ClusterTransformIndex);
	}

	if (LocalConnectionMethod == EClusterUnionMethod::PointImplicitAugmentedWithMinimalDelaunay ||
		LocalConnectionMethod == EClusterUnionMethod::MinimalSpanningSubsetDelaunayTriangulation)
	{
		FixConnectivityGraphUsingDelaunayTriangulation(Graph, Collection, ClusterTransformIndex);
	}
	
	return Graph;
}

void FGeometryCollectionConnectionGraphGenerator::ConnectChildren(FConnectionGraph& Graph, const TArray<int32>& Children, int32 ChildIndexA, int32 ChildIndexB)
{
	const int32 ChildTransformIndexA = Children[ChildIndexA];
	const int32 ChildTransformIndexB = Children[ChildIndexB];
	Graph[ChildIndexA].Add(ChildTransformIndexB);
	Graph[ChildIndexB].Add(ChildTransformIndexA);
}

FGeometryCollectionConnectionGraphGenerator::FVoronoiNeighbors
FGeometryCollectionConnectionGraphGenerator::ComputeVoronoiNeighbors(const FGeometryCollection& Collection, int32 ClusterTransformIndex)
{
	FVoronoiNeighbors Neighbors;

	const TManagedArray<FTransform3f>& Transforms = Collection.Transform;
	const TManagedArray<FTransform>& MassToLocal = Collection.GetAttribute<FTransform>("MassToLocal", FGeometryCollection::TransformGroup);

	const TSet<int32>& Children = Collection.Children[ClusterTransformIndex];

	Neighbors.Points.Reserve(Children.Num());
	
	for (const int32& ChildTransformIndex: Children)
	{
		const FTransform ChildTransform = MassToLocal[ChildTransformIndex] * FTransform(Transforms[ChildTransformIndex]);
		Neighbors.Points.Add(ChildTransform.GetLocation());
	}
	
	VoronoiNeighbors(Neighbors.Points, Neighbors.Neighbors);
	
	return Neighbors;
}

void FGeometryCollectionConnectionGraphGenerator::ComputeConnectivityGraphUsingDelaunayTriangulation(FConnectionGraph& Graph, const FGeometryCollection& Collection, int32 ClusterTransformIndex)
{
	const FVoronoiNeighbors VoronoiNeighbors = ComputeVoronoiNeighbors(Collection, ClusterTransformIndex);
	
	const TArray<int32> Children = Collection.Children[ClusterTransformIndex].Array();

	for (int32 i = 0; i < VoronoiNeighbors.Neighbors.Num(); i++)
	{
		for (int32 j = 0; j < VoronoiNeighbors.Neighbors[i].Num(); j++)
		{
			const int32 ChildIndexA = i;
			const int32 ChildIndexB = VoronoiNeighbors.Neighbors[i][j];
			ConnectChildren(Graph, Children, ChildIndexA, ChildIndexB);
		}
	}
}

void FGeometryCollectionConnectionGraphGenerator::FixConnectivityGraphUsingDelaunayTriangulation(FConnectionGraph& Graph, const FGeometryCollection& Collection, int32 ClusterTransformIndex)
{
	const TArray<int32> Children = Collection.Children[ClusterTransformIndex].Array();

	const FVoronoiNeighbors VoronoiNeighbors = ComputeVoronoiNeighbors(Collection, ClusterTransformIndex);
	
	// Build a UnionFind graph to find (indirectly) connected children
	using FGroupId = int32; 
	struct UnionFindInfo
	{
		FGroupId GroupId;
		int32 Size;
	};
	TMap<int32, UnionFindInfo> UnionInfo;
	UnionInfo.Reserve(Children.Num());

	// Initialize UnionInfo:
	//		0: GroupId = Children[0], Size = 1
	//		1: GroupId = Children[1], Size = 1
	//		2: GroupId = Children[2], Size = 1
	//		3: GroupId = Children[3], Size = 1
	for(int32 ChildTransformIndex : Children)
	{
		UnionInfo.Add(ChildTransformIndex, { ChildTransformIndex, 1 }); // GroupId, Size
	}

	auto FindGroup = [&](int32 ChildTransformIndex) 
	{
		FGroupId GroupId = ChildTransformIndex;
		if (GroupId)
		{
			int findIters = 0;
			while (UnionInfo[GroupId].GroupId != GroupId)
			{
				ensure(findIters++ < 10); // if this while loop iterates more than a few times, there's probably a bug in the unionfind
				auto& CurrInfo = UnionInfo[GroupId];
				auto& NextInfo = UnionInfo[CurrInfo.GroupId];
				CurrInfo.GroupId = NextInfo.GroupId;
				GroupId = NextInfo.GroupId;
				if (!GroupId) break; // error condidtion
			}
		}
		return GroupId;
	};

	// MergeGroup(Children[0], Children[1])
	//		0: GroupId = Children[1], Size = 0
	//		1: GroupId = Children[1], Size = 2
	//		2: GroupId = Children[2], Size = 1
	//		3: GroupId = Children[3], Size = 1
	auto MergeGroup = [&](int32 ChildA, int32 ChildB) 
	{
		FGroupId GroupA = FindGroup(ChildA);
		FGroupId GroupB = FindGroup(ChildB);
		if (GroupA == GroupB)
		{
			return;
		}
		// Make GroupA the smaller of the two
		if (UnionInfo[GroupA].Size > UnionInfo[GroupB].Size)
		{
			Swap(GroupA, GroupB);
		}
		// Overwrite GroupA with GroupB
		UnionInfo[GroupA].GroupId = GroupB;
		UnionInfo[GroupB].Size += UnionInfo[GroupA].Size;
		UnionInfo[GroupA].Size = 0; // not strictly necessary, but more correct
	};

	// Merge all groups with edges connecting them.
	for (int32 i = 0; i < Children.Num(); ++i)
	{
		const int32 ChildTransformIndex = Children[i]; 
		const FConnections& Siblings = Graph[i];
		for (const int32 SiblingTransformIndex : Siblings)
		{
			if (UnionInfo.Contains(SiblingTransformIndex))
			{
				MergeGroup(ChildTransformIndex, SiblingTransformIndex);
			}
		}
	}

	// Find candidate edges from the Delaunay graph to consider adding
	struct LinkCandidate
	{
		int32 A; // index in the Children array
		int32 B; // index in the Children array
		Chaos::FReal DistSq;
	};
	TArray<LinkCandidate> Candidates;
	Candidates.Reserve(VoronoiNeighbors.Neighbors.Num());

	// todo(chaos) need to change that 
	constexpr Chaos::FReal MClusterConnectionFactor = 1.0;
	constexpr Chaos::FReal AlwaysAcceptBelowDistSqThreshold = 50.0*50.0*100.0*MClusterConnectionFactor;
	for (int32 ChildIndex1 = 0; ChildIndex1 < VoronoiNeighbors.Neighbors.Num(); ChildIndex1++)
	{
		const int32 ChildTransformIndex1 = Children[ChildIndex1];
		const TArray<int32>& Child1Neighbors = VoronoiNeighbors.Neighbors[ChildIndex1];
		for (const int32 ChildIndex2 : Child1Neighbors)
		{
			if (ChildIndex2 < ChildIndex1)
			{
				// assume we'll get the symmetric connection; don't bother considering this one
				continue;
			}
			const int32 ChildTransformIndex2 = Children[ChildIndex2];

			const Chaos::FReal DistSq = FVector::DistSquared(VoronoiNeighbors.Points[ChildIndex1], VoronoiNeighbors.Points[ChildIndex2]);
			if (DistSq < AlwaysAcceptBelowDistSqThreshold)
			{
				// below always-accept threshold: don't bother adding to candidates array, just merge now
				MergeGroup(ChildTransformIndex1, ChildTransformIndex2);
				ConnectChildren(Graph, Children, ChildIndex1, ChildIndex2);
				continue;
			}

			if (FindGroup(ChildTransformIndex1) == FindGroup(ChildTransformIndex2))
			{
				// already part of the same group so we don't need Delaunay edge  
				continue;
			}

			// add to array to sort and add as-needed
			Candidates.Add({ ChildIndex1, ChildIndex2, DistSq });
		}
	}

	// Only add edges that would connect disconnected components, considering shortest edges first
	Candidates.Sort([](const LinkCandidate& A, const LinkCandidate& B) { return A.DistSq < B.DistSq; });
	for (const LinkCandidate& Candidate : Candidates)
	{
		const int32 ChildTransformIndex1 = Children[Candidate.A];
		const int32 ChildTransformIndex2 = Children[Candidate.B];
		if (FindGroup(ChildTransformIndex1) != FindGroup(ChildTransformIndex2))
		{
			MergeGroup(ChildTransformIndex1, ChildTransformIndex2);
			ConnectChildren(Graph, Children, Candidate.A, Candidate.B);
		}
	}
}

void FGeometryCollectionConnectionGraphGenerator::UpdateConnectivityGraphUsingPointImplicit(FConnectionGraph& Graph, const FGeometryCollection& Collection, int32 ClusterTransformIndex)
{
	// const TSet<int32>& Children = Collection.Children[ClusterTransformIndex];
	//
	//
	// const TManagedArray<FTransform>& Transforms = Collection.Transform;
	// const TManagedArray<FTransform>& MassToLocal = Collection.GetAttribute<FTransform>("MassToLocal", FGeometryCollection::TransformGroup);
	//
	// const FReal Delta = FMath::Min(FMath::Max(Parameters.CoillisionThicknessPercent, FReal(0)), FReal(1));
	//
	// Graph.SetNum(Children.Num());
	// Chaos::PhysicsParallelFor(Children.Num(), [&](int32 Child1)
	// {
	// 	const int32 ChildTransformIndex1 = Children[FSetElementId::FromInteger(Child1)];
	// 	const FBox& ChildBoundingBox = Collection.BoundingBox[ChildTransformIndex1];
	// 	const FTransform ChildTransform1 = MassToLocal[ChildTransformIndex1] * Transforms[ChildTransformIndex1];
	// 	if (Collection.TransformToGeometryIndex[ChildTransformIndex1] > INDEX_NONE)
	// 	{
	// 		FConnections& Connections = Graph[Child1];
	//
	// 		const int32 Offset = Child1 + 1;
	// 		const int32 NumRemainingChildren = Children.Num() - Offset;
	//
	// 		for (int32 Idx = 0; Idx < NumRemainingChildren; ++Idx)
	// 		{
	// 			const int32 Child2 = Offset + Idx;
	// 			
	// 			const int32 ChildTransformIndex2 = Children[Child2];
	// 			if (Child2Particle->CollisionParticles())
	// 			{
	// 				const FTransform ChildTransform2 = MassToLocal[ChildTransformIndex2] * Transforms[ChildTransformIndex2];
	// 				const uint32 NumCollisionParticles = Child2Particle->CollisionParticles()->Size();
	// 				for (uint32 CollisionIdx = 0; CollisionIdx < NumCollisionParticles; ++CollisionIdx)
	// 				{
	// 					const FVector LocalPoint =
	// 						ChildTransform2.TransformPositionNoScale(Child2Particle->CollisionParticles()->X(CollisionIdx));
	// 					const Chaos::FReal Phi = Child1->Geometry()->SignedDistance(LocalPoint - (LocalPoint * Delta));
	// 					if (Phi < 0.0)
	// 					{
	// 						Connections.Add(ChildTransformIndex2);
	// 						break;
	// 					}
	//
	// 				}
	// 			}
	// 		}
	// 	}
	// });
	//
	// // join results and make connections
	// for (const ParticlePairArray& ConnectionList : Connections)
	// {
	// 	for (const ParticlePair& Edge : ConnectionList)
	// 	{
	// 		ConnectNodes(Edge.Key, Edge.Value);
	// 	}
	// }
}

void FGeometryCollectionConnectionGraphGenerator::CommitToCollection(FConnectionGraph& Graph, FGeometryCollection& Collection, int32 ClusterTransformIndex) 
{
	GeometryCollection::Facades::FCollectionConnectionGraphFacade ConnectionFacade(Collection);
	ConnectionFacade.DefineSchema();

	for (int32 ChildTransformIndex : Collection.Children[ClusterTransformIndex])
	{
		ConnectionFacade.ReserveAdditionalConnections(Graph[ChildTransformIndex].Num());
		for (int32 ChildNbr : Graph[ChildTransformIndex])
		{
			ConnectionFacade.Connect(ChildTransformIndex, ChildNbr);
		}
	}
}
