// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionConvexPropertiesInterface.h"
#include "GeometryCollection/GeometryCollection.h"

const FName FGeometryCollectionConvexPropertiesInterface::ConvexPropertiesGroup = "ConvexProperties";
const FName FGeometryCollectionConvexPropertiesInterface::ConvexIndexAttribute = "TransformGroupIndex";
const FName FGeometryCollectionConvexPropertiesInterface::ConvexEnable = "Enable";
const FName FGeometryCollectionConvexPropertiesInterface::ConvexFractionRemoveAttribute = "FractionRemove";
const FName FGeometryCollectionConvexPropertiesInterface::ConvexSimplificationThresholdAttribute = "SimplificationThreshold";
const FName FGeometryCollectionConvexPropertiesInterface::ConvexCanExceedFractionAttribute = "CanExceedFraction";
const FName FGeometryCollectionConvexPropertiesInterface::ConvexRemoveOverlapsMethodAttribute = "RemoveOverlapsMethod";
const FName FGeometryCollectionConvexPropertiesInterface::ConvexRemoveOverlapsShrinkAttribute = "RemoveOverlapsShrinkPercent";

FGeometryCollectionConvexPropertiesInterface::FGeometryCollectionConvexPropertiesInterface(FManagedArrayCollection* InGeometryCollection)
	: FManagedArrayInterface(InGeometryCollection)
{}

void 
FGeometryCollectionConvexPropertiesInterface::InitializeInterface()
{
	if (!ManagedCollection->HasGroup(ConvexPropertiesGroup))
	{
		ManagedCollection->AddGroup(ConvexPropertiesGroup);
	}

	if (!ManagedCollection->HasAttribute(ConvexIndexAttribute, ConvexPropertiesGroup))
	{
		FManagedArrayCollection::FConstructionParameters AttributeDependency(FTransformCollection::TransformGroup);
		ManagedCollection->AddAttribute<int32>(ConvexIndexAttribute, ConvexPropertiesGroup, AttributeDependency);
	}

	if (!ManagedCollection->HasAttribute(ConvexEnable, ConvexPropertiesGroup))
	{
		ManagedCollection->AddAttribute<bool>(ConvexEnable, ConvexPropertiesGroup);
	}

	if (!ManagedCollection->HasAttribute(ConvexFractionRemoveAttribute, ConvexPropertiesGroup))
	{
		ManagedCollection->AddAttribute<float>(ConvexFractionRemoveAttribute, ConvexPropertiesGroup);
	}

	if (!ManagedCollection->HasAttribute(ConvexSimplificationThresholdAttribute, ConvexPropertiesGroup))
	{
		ManagedCollection->AddAttribute<float>(ConvexSimplificationThresholdAttribute, ConvexPropertiesGroup);
	}

	if (!ManagedCollection->HasAttribute(ConvexCanExceedFractionAttribute, ConvexPropertiesGroup))
	{
		ManagedCollection->AddAttribute<float>(ConvexCanExceedFractionAttribute, ConvexPropertiesGroup);
	}

	if (!ManagedCollection->HasAttribute(ConvexRemoveOverlapsMethodAttribute, ConvexPropertiesGroup))
	{
		ManagedCollection->AddAttribute<int32>(ConvexRemoveOverlapsMethodAttribute, ConvexPropertiesGroup);
	}

	if (!ManagedCollection->HasAttribute(ConvexRemoveOverlapsShrinkAttribute, ConvexPropertiesGroup))
	{
		ManagedCollection->AddAttribute<float>(ConvexRemoveOverlapsShrinkAttribute, ConvexPropertiesGroup);
	}

	SetDefaultProperty();
}

void FGeometryCollectionConvexPropertiesInterface::SetDefaultProperty()
{
	FConvexCreationProperties DefaultProperty;
	TManagedArray<int32>& Index = ManagedCollection->ModifyAttribute<int32>(ConvexIndexAttribute, ConvexPropertiesGroup);
	if (Index.Num() == 0 || !Index.Contains(INDEX_NONE))
	{
		int32 DefaultAttributeIndex = ManagedCollection->AddElements(1, ConvexPropertiesGroup);
		Index[DefaultAttributeIndex] = INDEX_NONE;
		SetConvexProperties(DefaultProperty);
	}
}

FGeometryCollectionConvexPropertiesInterface::FConvexCreationProperties 
FGeometryCollectionConvexPropertiesInterface::GetDefaultProperty() const
{
	FConvexCreationProperties DefaultProperty;
	const TManagedArray<int32>& Index = ManagedCollection->GetAttribute<int32>(ConvexIndexAttribute, ConvexPropertiesGroup);
	if (Index.Contains(INDEX_NONE))
	{
		const TManagedArray<bool>& Enable = ManagedCollection->GetAttribute<bool>(ConvexEnable, ConvexPropertiesGroup);
		const TManagedArray<float>& FractionRemove = ManagedCollection->GetAttribute<float>(ConvexFractionRemoveAttribute, ConvexPropertiesGroup);
		const TManagedArray<float>& SimplificationThreshold = ManagedCollection->GetAttribute<float>(ConvexSimplificationThresholdAttribute, ConvexPropertiesGroup);
		const TManagedArray<float>& CanExceedFraction = ManagedCollection->GetAttribute<float>(ConvexCanExceedFractionAttribute, ConvexPropertiesGroup);
		const TManagedArray<int32>& RemoveOverlaps = ManagedCollection->GetAttribute<int32>(ConvexRemoveOverlapsMethodAttribute, ConvexPropertiesGroup);
		const TManagedArray<float>& RemoveOverlapsShrinkPercent = ManagedCollection->GetAttribute<float>(ConvexRemoveOverlapsShrinkAttribute, ConvexPropertiesGroup);

		int32 DefaultIndex = Index.Find(INDEX_NONE);
		if (0 <= DefaultIndex && DefaultIndex < Index.Num())
		{
			DefaultProperty.Enable = Enable[DefaultIndex];
			DefaultProperty.FractionRemove = FractionRemove[DefaultIndex];
			DefaultProperty.SimplificationThreshold = SimplificationThreshold[DefaultIndex];
			DefaultProperty.CanExceedFraction = CanExceedFraction[DefaultIndex];
			DefaultProperty.RemoveOverlaps = (EConvexOverlapRemoval)RemoveOverlaps[DefaultIndex];
			DefaultProperty.OverlapRemovalShrinkPercent = RemoveOverlapsShrinkPercent[DefaultIndex];
		}
	}
	return DefaultProperty;
}

void 
FGeometryCollectionConvexPropertiesInterface::CleanInterfaceForCook()
{
	RemoveInterfaceAttributes();
}

void 
FGeometryCollectionConvexPropertiesInterface::RemoveInterfaceAttributes()
{
	ManagedCollection->RemoveGroup(ConvexPropertiesGroup);

}

FGeometryCollectionConvexPropertiesInterface::FConvexCreationProperties
FGeometryCollectionConvexPropertiesInterface::GetConvexProperties(int TransformGroupIndex) const
{
	FConvexCreationProperties ConvexProperty = GetDefaultProperty();
	const TManagedArray<int32>& Index = ManagedCollection->GetAttribute<int32>(ConvexIndexAttribute, ConvexPropertiesGroup);
	int32 PropIndex = Index.Find(TransformGroupIndex);
	if (0 <= PropIndex && PropIndex < Index.Num())
	{
		const TManagedArray<bool>& Enable = ManagedCollection->GetAttribute<bool>(ConvexEnable, ConvexPropertiesGroup);
		const TManagedArray<float>& FractionRemove = ManagedCollection->GetAttribute<float>(ConvexFractionRemoveAttribute, ConvexPropertiesGroup);
		const TManagedArray<float>& SimplificationThreshold = ManagedCollection->GetAttribute<float>(ConvexSimplificationThresholdAttribute, ConvexPropertiesGroup);
		const TManagedArray<float>& CanExceedFraction = ManagedCollection->GetAttribute<float>(ConvexCanExceedFractionAttribute, ConvexPropertiesGroup);
		const TManagedArray<int32>& RemoveOverlaps = ManagedCollection->GetAttribute<int32>(ConvexRemoveOverlapsMethodAttribute, ConvexPropertiesGroup);
		const TManagedArray<float>& RemoveOverlapsShrinkPercent = ManagedCollection->GetAttribute<float>(ConvexRemoveOverlapsShrinkAttribute, ConvexPropertiesGroup);

		ConvexProperty.Enable = Enable[PropIndex];
		ConvexProperty.FractionRemove = FractionRemove[PropIndex];
		ConvexProperty.SimplificationThreshold = SimplificationThreshold[PropIndex];
		ConvexProperty.CanExceedFraction = CanExceedFraction[PropIndex];
		ConvexProperty.RemoveOverlaps = (EConvexOverlapRemoval)RemoveOverlaps[PropIndex];
		ConvexProperty.OverlapRemovalShrinkPercent = RemoveOverlapsShrinkPercent[PropIndex];
	}
	return ConvexProperty;
}

void
FGeometryCollectionConvexPropertiesInterface::SetConvexProperties(const FConvexCreationProperties& InConvexAttributes, int TransformGroupIndex)
{
	FConvexCreationProperties Property;
	TManagedArray<int32>& Index = ManagedCollection->ModifyAttribute<int32>(ConvexIndexAttribute, ConvexPropertiesGroup);
	TManagedArray<bool>& Enable = ManagedCollection->ModifyAttribute<bool>(ConvexEnable, ConvexPropertiesGroup);
	TManagedArray<float>& FractionRemove = ManagedCollection->ModifyAttribute<float>(ConvexFractionRemoveAttribute, ConvexPropertiesGroup);
	TManagedArray<float>& SimplificationThreshold = ManagedCollection->ModifyAttribute<float>(ConvexSimplificationThresholdAttribute, ConvexPropertiesGroup);
	TManagedArray<float>& CanExceedFraction = ManagedCollection->ModifyAttribute<float>(ConvexCanExceedFractionAttribute, ConvexPropertiesGroup);
	TManagedArray<int32>& RemoveOverlaps = ManagedCollection->ModifyAttribute<int32>(ConvexRemoveOverlapsMethodAttribute, ConvexPropertiesGroup);
	TManagedArray<float>& RemoveOverlapsShrinkPercent = ManagedCollection->ModifyAttribute<float>(ConvexRemoveOverlapsShrinkAttribute, ConvexPropertiesGroup);

	int32 AttributeIndex = INDEX_NONE;
	if (!Index.Contains(TransformGroupIndex))
	{
		AttributeIndex = ManagedCollection->AddElements(1, ConvexPropertiesGroup);
	}
	else
	{
		AttributeIndex = Index.Find(TransformGroupIndex);
	}

	if (0 <= AttributeIndex && AttributeIndex < Index.Num())
	{
		Index[AttributeIndex] = TransformGroupIndex;
		Enable[AttributeIndex] = InConvexAttributes.Enable;
		FractionRemove[AttributeIndex] = InConvexAttributes.FractionRemove;
		SimplificationThreshold[AttributeIndex] = InConvexAttributes.SimplificationThreshold;
		CanExceedFraction[AttributeIndex] = InConvexAttributes.CanExceedFraction;
		RemoveOverlaps[AttributeIndex] = (int32)InConvexAttributes.RemoveOverlaps;
		RemoveOverlapsShrinkPercent[AttributeIndex] = InConvexAttributes.OverlapRemovalShrinkPercent;
	}
}
