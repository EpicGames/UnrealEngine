// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionConvexUtility.h"
#include "Chaos/Convex.h"
#include "Chaos/GJK.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/GeometryCollectionProximityUtility.h"
#include "GeometryCollection/Facades/CollectionTransformFacade.h"
#include "GeometryCollection/Facades/CollectionHierarchyFacade.h"
#include "GeometryCollection/Facades/CollectionMeshFacade.h"
#include "CompGeom/ConvexHull3.h"
#include "Templates/Sorting.h"
#include "Spatial/PointHashGrid3.h"
#include "Operations/MeshBoolean.h"
#include "Operations/MeshSelfUnion.h"
#include "MeshQueries.h"
#include "GeometryCollection/GeometryCollectionClusteringUtility.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionConvexUtility)

bool UseVolumeToComputeRelativeSize = false;
FAutoConsoleVariableRef CVarUseVolumeToComputeRelativeSize(TEXT("p.gc.UseVolumeToComputeRelativeSize"), UseVolumeToComputeRelativeSize, TEXT("Use Volume To Compute RelativeSize instead of the side of the cubic volume (def: false)"));

bool UseLargestClusterToComputeRelativeSize = false;
FAutoConsoleVariableRef CVarUseMaxClusterToComputeRelativeSize(TEXT("p.gc.UseLargestClusterToComputeRelativeSize"), UseVolumeToComputeRelativeSize, TEXT("Use the largest Cluster as reference for the releative size instead of the largest child (def: false)"));

static const Chaos::FVec3f IcoSphere_Subdiv0[] =
{
	{  0.000000f,  0.000000f, -1.000000f },
	{  0.525720f,  0.723600f, -0.447215f },
	{  0.850640f, -0.276385f, -0.447215f },
	{  0.000000f, -0.894425f, -0.447215f },
	{ -0.850640f, -0.276385f, -0.447215f },
	{ -0.525720f,  0.723600f, -0.447215f },
	{  0.850640f,  0.276385f,  0.447215f },
	{  0.525720f, -0.723600f,  0.447215f },
	{ -0.525720f, -0.723600f,  0.447215f },
	{ -0.850640f,  0.276385f,  0.447215f },
	{  0.000000f,  0.894425f,  0.447215f },
	{  0.000000f,  0.000000f,  1.000000f },
};
constexpr int32 IcoSphere_Subdiv0_Num = sizeof(IcoSphere_Subdiv0) / sizeof(Chaos::FVec3f);

static const Chaos::FVec3f IcoSphere_Subdiv1[] =
{
	{  0.000000f,  0.000000f, -1.000000f },
	{  0.525725f,  0.723607f, -0.447220f },
	{  0.850649f, -0.276388f, -0.447220f },
	{  0.000000f, -0.894426f, -0.447216f },
	{ -0.850649f, -0.276388f, -0.447220f },
	{ -0.525725f,  0.723607f, -0.447220f },
	{  0.850649f,  0.276388f,  0.447220f },
	{  0.525725f, -0.723607f,  0.447220f },
	{ -0.525725f, -0.723607f,  0.447220f },
	{ -0.850649f,  0.276388f,  0.447220f },
	{  0.000000f,  0.894426f,  0.447216f },
	{  0.000000f,  0.000000f,  1.000000f },
	{  0.499995f, -0.162456f, -0.850654f },
	{  0.309011f,  0.425323f, -0.850654f },
	{  0.809012f,  0.262869f, -0.525738f },
	{  0.000000f,  0.850648f, -0.525736f },
	{ -0.309011f,  0.425323f, -0.850654f },
	{  0.000000f, -0.525730f, -0.850652f },
	{  0.499997f, -0.688189f, -0.525736f },
	{ -0.499995f, -0.162456f, -0.850654f },
	{ -0.499997f, -0.688189f, -0.525736f },
	{ -0.809012f,  0.262869f, -0.525738f },
	{  0.309013f,  0.951058f,  0.000000f },
	{ -0.309013f,  0.951058f,  0.000000f },
	{  1.000000f,  0.000000f,  0.000000f },
	{  0.809017f,  0.587786f,  0.000000f },
	{  0.309013f, -0.951058f,  0.000000f },
	{  0.809017f, -0.587786f,  0.000000f },
	{ -0.809017f, -0.587786f,  0.000000f },
	{ -0.309013f, -0.951058f,  0.000000f },
	{ -0.809017f,  0.587786f,  0.000000f },
	{ -1.000000f,  0.000000f,  0.000000f },
	{  0.499997f,  0.688189f,  0.525736f },
	{  0.809012f, -0.262869f,  0.525738f },
	{  0.000000f, -0.850648f,  0.525736f },
	{ -0.809012f, -0.262869f,  0.525738f },
	{ -0.499997f,  0.688189f,  0.525736f },
	{  0.499995f,  0.162456f,  0.850654f },
	{  0.000000f,  0.525730f,  0.850652f },
	{  0.309011f, -0.425323f,  0.850654f },
	{ -0.309011f, -0.425323f,  0.850654f },
	{ -0.499995f,  0.162456f,  0.850654f },
};
constexpr int32 IcoSphere_Subdiv1_Num = sizeof(IcoSphere_Subdiv1) / sizeof(Chaos::FVec3f);

static const Chaos::FVec3f IcoHemisphere_Subdiv1[] =
{
	{  0.850649f,  0.27638f, 0.447220f },
	{  0.525725f, -0.72360f, 0.447220f },
	{ -0.525725f, -0.72360f, 0.447220f },
	{ -0.850649f,  0.27638f, 0.447220f },
	{  0.000000f,  0.89442f, 0.447216f },
	{  0.000000f,  0.00000f, 1.000000f },
	{  0.309013f,  0.95105f, 0.000000f },
	{ -0.309013f,  0.95105f, 0.000000f },
	{  1.000000f,  0.00000f, 0.000000f },
	{  0.809017f,  0.58778f, 0.000000f },
	{  0.309013f, -0.95105f, 0.000000f },
	{  0.809017f, -0.58778f, 0.000000f },
	{ -0.809017f, -0.58778f, 0.000000f },
	{ -0.309013f, -0.95105f, 0.000000f },
	{ -0.809017f,  0.58778f, 0.000000f },
	{ -1.000000f,  0.00000f, 0.000000f },
	{  0.499997f,  0.68818f, 0.525736f },
	{  0.809012f, -0.26286f, 0.525738f },
	{  0.000000f, -0.85064f, 0.525736f },
	{ -0.809012f, -0.26286f, 0.525738f },
	{ -0.499997f,  0.68818f, 0.525736f },
	{  0.499995f,  0.16245f, 0.850654f },
	{  0.000000f,  0.52573f, 0.850652f },
	{  0.309011f, -0.42532f, 0.850654f },
	{ -0.309011f, -0.42532f, 0.850654f },
	{ -0.499995f,  0.16245f, 0.850654f },
};
constexpr int32 IcoHemisphere_Subdiv1_Num = sizeof(IcoHemisphere_Subdiv1) / sizeof(Chaos::FVec3f);


TOptional<FGeometryCollectionConvexUtility::FGeometryCollectionConvexData> FGeometryCollectionConvexUtility::GetConvexHullDataIfPresent(FManagedArrayCollection* GeometryCollection)
{
	check(GeometryCollection);

	if (!GeometryCollection->HasAttribute("TransformToConvexIndices", FTransformCollection::TransformGroup) ||
		!GeometryCollection->HasAttribute(FGeometryCollection::ConvexHullAttribute, FGeometryCollection::ConvexGroup))
	{
		return TOptional<FGeometryCollectionConvexUtility::FGeometryCollectionConvexData>();
	}

	FGeometryCollectionConvexUtility::FGeometryCollectionConvexData ConvexData{
		GeometryCollection->ModifyAttribute<TSet<int32>>("TransformToConvexIndices", FTransformCollection::TransformGroup),
		GeometryCollection->ModifyAttribute<Chaos::FConvexPtr>(FGeometryCollection::ConvexHullAttribute, FGeometryCollection::ConvexGroup)
	};
	return TOptional<FGeometryCollectionConvexUtility::FGeometryCollectionConvexData>(ConvexData);
}

bool FGeometryCollectionConvexUtility::HasConvexHullData(const FManagedArrayCollection* Collection)
{
	return Collection->HasAttribute("TransformToConvexIndices", FTransformCollection::TransformGroup) && Collection->HasAttribute(FGeometryCollection::ConvexHullAttribute, FGeometryCollection::ConvexGroup);
}

FGeometryCollectionConvexUtility::FGeometryCollectionConvexData FGeometryCollectionConvexUtility::GetValidConvexHullData(FGeometryCollection* GeometryCollection)
{
	check(GeometryCollection)

	CreateConvexHullAttributesIfNeeded(*GeometryCollection);

	// Check for correct population. Make sure all rigid nodes should have a convex associated; leave convex hulls for transform nodes alone for now
	const TManagedArray<int32>& SimulationType = GeometryCollection->GetAttribute<int32>("SimulationType", FTransformCollection::TransformGroup);
	const TManagedArray<int32>& TransformToGeometryIndex = GeometryCollection->GetAttribute<int32>("TransformToGeometryIndex", FTransformCollection::TransformGroup);
	TManagedArray<TSet<int32>>& TransformToConvexIndices = GeometryCollection->ModifyAttribute<TSet<int32>>("TransformToConvexIndices", FTransformCollection::TransformGroup);
	TManagedArray<Chaos::FConvexPtr>& ConvexHull = GeometryCollection->ModifyAttribute<Chaos::FConvexPtr>(FGeometryCollection::ConvexHullAttribute, FGeometryCollection::ConvexGroup);
	
	TArray<int32> ProduceConvexHulls;
	ProduceConvexHulls.Reserve(SimulationType.Num());

	for (int32 Idx = 0; Idx < SimulationType.Num(); ++Idx)
	{
		if ((SimulationType[Idx] == FGeometryCollection::ESimulationTypes::FST_Rigid) && (TransformToConvexIndices[Idx].Num() == 0))
		{
			ProduceConvexHulls.Add(Idx);
		}
	}

	if (ProduceConvexHulls.Num())
	{
		int32 NewConvexIndexStart = GeometryCollection->AddElements(ProduceConvexHulls.Num(), FGeometryCollection::ConvexGroup);
		for (int32 Idx = 0; Idx < ProduceConvexHulls.Num(); ++Idx)
		{
			int32 GeometryIdx = TransformToGeometryIndex[ProduceConvexHulls[Idx]];
			ConvexHull[NewConvexIndexStart + Idx] = GetConvexHull(GeometryCollection, GeometryIdx);
			TransformToConvexIndices[ProduceConvexHulls[Idx]].Reset();
			TransformToConvexIndices[ProduceConvexHulls[Idx]].Add(NewConvexIndexStart + Idx);
		}
	}

	return { TransformToConvexIndices, ConvexHull };
}



namespace
{

typedef Chaos::TPlaneConcrete<Chaos::FReal, 3> FChaosPlane;

// filter points s.t. they are spaced at least more than SimplificationDistanceThreshold apart (after starting with the 4 'extreme' points to ensure we cover a volume)
void FilterHullPoints(const TArray<Chaos::FConvex::FVec3Type>& InPts, TArray<Chaos::FConvex::FVec3Type>& OutPts, double SimplificationDistanceThreshold)
{
	if (SimplificationDistanceThreshold > 0 && InPts.Num() > 0)
	{
		OutPts.Reset();

		int32 NumPts = InPts.Num();
		TArray<Chaos::FReal> DistSq;
		DistSq.SetNumUninitialized(NumPts);
		TArray<int32> PointOrder;
		PointOrder.SetNumUninitialized(NumPts);
		UE::Geometry::TPointHashGrid3<int32, Chaos::FReal> Spatial((Chaos::FReal)SimplificationDistanceThreshold, INDEX_NONE);
		Chaos::FAABB3 Bounds;
		for (int32 VIdx = 0; VIdx < NumPts; VIdx++)
		{
			Bounds.GrowToInclude(InPts[VIdx]);
			PointOrder[VIdx] = VIdx;
		}

		// Rank points by squared distance from center
		Chaos::FVec3 Center = Bounds.Center();
		for (int i = 0; i < NumPts; i++)
		{
			DistSq[i] = (InPts[i] - Center).SizeSquared();
		}

		// Start by picking the 'extreme' points to ensure we cover the volume reasonably (otherwise it's too easy to end up with a degenerate hull pieces)
		UE::Geometry::TExtremePoints3<Chaos::FReal> ExtremePoints(NumPts,
			[&InPts](int32 Idx)->UE::Math::TVector<Chaos::FReal> 
			{ 
				return (UE::Math::TVector<Chaos::FReal>)InPts[Idx];
			});
		for (int32 ExtremeIdx = 0; ExtremeIdx < ExtremePoints.Dimension + 1; ExtremeIdx++)
		{
			int32 ExtremePtIdx = ExtremePoints.Extreme[ExtremeIdx];
			if (!InPts.IsValidIndex(ExtremePtIdx))
			{
				break;
			}
			Chaos::FVec3 ExtremePt = InPts[ExtremePtIdx];
			OutPts.Add(ExtremePt);
			Spatial.InsertPointUnsafe(ExtremePtIdx, ExtremePt);
			DistSq[ExtremePtIdx] = -1; // remove these points from the distance ranking
		}

		// Sort in descending order
		Algo::Sort(PointOrder, [&DistSq](int32 i, int32 j)
			{
				return DistSq[i] > DistSq[j];
			});

		// Filter to include only w/ no other too-close points, prioritizing the farthest points
		for (int32 OrdIdx = 0; OrdIdx < NumPts; OrdIdx++)
		{
			int32 PtIdx = PointOrder[OrdIdx];
			if (DistSq[PtIdx] < 0)
			{
				break; // we've reached the extreme points that were already selected
			}
			Chaos::FVec3 Pt = InPts[PtIdx];
			TPair<int32, Chaos::FReal> NearestIdx = Spatial.FindNearestInRadius(Pt, (Chaos::FReal)SimplificationDistanceThreshold,
				[&InPts, &Pt](int32 Idx)
				{
					return (Pt - InPts[Idx]).SizeSquared();
				});
			if (NearestIdx.Key == INDEX_NONE)
			{
				Spatial.InsertPointUnsafe(PtIdx, Pt);
				OutPts.Add(Pt);
			}
		}
	}
	else
	{
		// No filtering requested -- in practice, we don't call this function in these cases below
		OutPts = InPts;
	}
}

void FilterHullPoints(TArray<Chaos::FConvex::FVec3Type>& InOutPts, double SimplificationDistanceThreshold)
{
	if (SimplificationDistanceThreshold > 0)
	{
		TArray<Chaos::FConvex::FVec3Type> FilteredPts;
		FilterHullPoints(InOutPts, FilteredPts, SimplificationDistanceThreshold);
		InOutPts = MoveTemp(FilteredPts);
	}
}

FVector ScaleHullPoints(TArray<Chaos::FConvex::FVec3Type>& InOutPts, double ShrinkPercentage)
{
	FVector Pivot = FVector::ZeroVector;
	if (ShrinkPercentage == 0.0 || InOutPts.IsEmpty())
	{
		return Pivot;
	}

	double ScaleFactor = 1.0 - ShrinkPercentage / 100.0;
	ensure(ScaleFactor != 0.0);

	FVector Sum(0,0,0);
	for (Chaos::FConvex::FVec3Type& Pt : InOutPts)
	{
		Sum += (FVector)Pt;
	}

	Pivot = Sum / (double)InOutPts.Num();
	for (Chaos::FConvex::FVec3Type& Pt : InOutPts)
	{
		Pt = Chaos::FConvex::FVec3Type((FVector(Pt) - Pivot) * ScaleFactor + Pivot);
	}

	return Pivot;
}

void AddUnscaledPts(TArray<Chaos::FConvex::FVec3Type>& OutHullPts, const TArray<Chaos::FConvex::FVec3Type>& Vertices, const FVector& Pivot, double ShrinkPercentage)
{
	double ScaleFactor = 1.0 - ShrinkPercentage / 100.0;
	double InvScaleFactor = 1.0;
	if (ensure(ScaleFactor != 0.0))
	{
		InvScaleFactor = 1.0 / ScaleFactor;
	}

	int32 StartIdx = OutHullPts.AddUninitialized(Vertices.Num());
	for (int32 VertIdx = 0, Num = Vertices.Num(); VertIdx < Num; VertIdx++)
	{
		FVector Vert = (FVector)Vertices[VertIdx];
		OutHullPts[StartIdx + VertIdx] = Chaos::FConvex::FVec3Type((Vert - Pivot) * InvScaleFactor + Pivot);
	}
}

Chaos::FConvex MakeHull(const TArray<Chaos::FConvex::FVec3Type>& Pts, double SimplificationDistanceThreshold)
{
	if (SimplificationDistanceThreshold > 0)
	{
		TArray<Chaos::FConvex::FVec3Type> FilteredPts;
		FilterHullPoints(Pts, FilteredPts, SimplificationDistanceThreshold);
		
		return Chaos::FConvex(FilteredPts, UE_KINDA_SMALL_NUMBER);
	}
	else
	{
		return Chaos::FConvex(Pts, UE_KINDA_SMALL_NUMBER);
	}
}

/// Cut hull with plane, generating the point set of a new hull
/// @return false if plane did not cut any points on the hull
bool CutHull(const Chaos::FConvex& HullIn, FChaosPlane Plane, bool KeepSide, TArray<Chaos::FConvex::FVec3Type>& HullPtsOut)
{
	const TArray<Chaos::FConvex::FVec3Type>& Vertices = HullIn.GetVertices();
	const Chaos::FConvexStructureData& HullData = HullIn.GetStructureData();
	bool bHasOutside = false;
	for (int VertIdx = 0; VertIdx < Vertices.Num(); VertIdx++)
	{
		const Chaos::FVec3& V = Vertices[VertIdx];
		if ((Plane.SignedDistance(V) < 0) == KeepSide)
		{
			HullPtsOut.Add(V);
		}
		else
		{
			bHasOutside = true;
		}
	}

	if (!bHasOutside)
	{
		return false;
	}

	int32 NumPlanes = HullIn.NumPlanes();
	for (int PlaneIdx = 0; PlaneIdx < NumPlanes; PlaneIdx++)
	{
		int32 NumPlaneVerts = HullData.NumPlaneVertices(PlaneIdx);
		for (int32 PlaneVertexIdx = 0; PlaneVertexIdx < NumPlaneVerts; PlaneVertexIdx++)
		{
			int32 NextVertIdx = (PlaneVertexIdx + 1) % NumPlaneVerts;
			const Chaos::FVec3& V0 = Vertices[HullData.GetPlaneVertex(PlaneIdx, PlaneVertexIdx)];
			const Chaos::FVec3& V1 = Vertices[HullData.GetPlaneVertex(PlaneIdx, NextVertIdx)];
			if ((Plane.SignedDistance(V0) < 0) != (Plane.SignedDistance(V1) < 0))
			{
				Chaos::Pair<Chaos::FVec3, bool> Res = Plane.FindClosestIntersection(V0, V1, 0);
				if (Res.Second)
				{
					HullPtsOut.Add(Res.First);
				}
			}
		}
	}

	return true;
}


/// Cut hull with plane, keeping both sides and generating the point set of both new hulls
/// @return false if plane did not cut any points on the hull
bool SplitHull(const Chaos::FConvex& HullIn, FChaosPlane Plane, bool KeepSide, TArray<Chaos::FVec3>& InsidePtsOut, TArray<Chaos::FVec3>& OutsidePtsOut)
{
	const TArray<Chaos::FConvex::FVec3Type>& Vertices = HullIn.GetVertices();
	const Chaos::FConvexStructureData& HullData = HullIn.GetStructureData();
	bool bHasOutside = false;
	for (int VertIdx = 0; VertIdx < Vertices.Num(); VertIdx++)
	{
		const Chaos::FVec3& V = Vertices[VertIdx];
		if ((Plane.SignedDistance(V) < 0) == KeepSide)
		{
			InsidePtsOut.Add(V);
		}
		else
		{
			OutsidePtsOut.Add(V);
			bHasOutside = true;
		}
	}

	if (!bHasOutside)
	{
		return false;
	}

	int32 NumPlanes = HullIn.NumPlanes();
	for (int PlaneIdx = 0; PlaneIdx < NumPlanes; PlaneIdx++)
	{
		int32 NumPlaneVerts = HullData.NumPlaneVertices(PlaneIdx);
		for (int32 PlaneVertexIdx = 0; PlaneVertexIdx < NumPlaneVerts; PlaneVertexIdx++)
		{
			int32 NextVertIdx = (PlaneVertexIdx + 1) % NumPlaneVerts;
			const Chaos::FVec3 V0 = Vertices[HullData.GetPlaneVertex(PlaneIdx, PlaneVertexIdx)];
			const Chaos::FVec3 V1 = Vertices[HullData.GetPlaneVertex(PlaneIdx, NextVertIdx)];
			if ((Plane.SignedDistance(V0) < 0) != (Plane.SignedDistance(V1) < 0))
			{
				Chaos::Pair<Chaos::FVec3, bool> Res = Plane.FindClosestIntersection(V0, V1, 0);
				if (Res.Second)
				{
					InsidePtsOut.Add(Res.First);
					OutsidePtsOut.Add(Res.First);
				}
			}
		}
	}

	return true;
}

// Add a pivot and verify (in debug) that the pivot array and convexes array are in sync
void AddPivot(TArray<Chaos::FConvexPtr>& Convexes, TArray<FVector>& ConvexPivots, FVector Pivot)
{
	ConvexPivots.Add(Pivot);
	checkSlow(Convexes.Num() == ConvexPivots.Num());
}

/// Assumptions: 
///		Convexes is initialized to one convex hull for each leaf geometry in a SHARED coordinate space
///		TransformToConvexIndices is initialized to point to the existing convex hulls
///		Parents, GeoProximity, and GeometryToTransformIndex are all initialized from the geometry collection
void CreateNonoverlappingConvexHulls(
	TArray<Chaos::FConvexPtr>& Convexes,
	TArray<FVector>& ConvexPivots,
	TArray<TSet<int32>>& TransformToConvexIndices,
	TFunctionRef<bool(int32)> HasCustomConvexFn,
	const TManagedArray<int32>& SimulationType,
	int32 LeafType,
	int32 SkipType,
	const TManagedArray<int32>& Parents,
	const TManagedArray<TSet<int32>>* GeoProximity,
	const TManagedArray<int32>& GeometryToTransformIndex,
	const TManagedArray<float>* Volume,
	double FracAllowRemove,
	double SimplificationDistanceThreshold,
	double CanExceedFraction,
	EConvexOverlapRemoval OverlapRemovalMethod,
	double ShrinkPercentage
)
{
	bool bRemoveOverlaps = OverlapRemovalMethod != EConvexOverlapRemoval::None;
	bool bRemoveLeafOverlaps = OverlapRemovalMethod == EConvexOverlapRemoval::All;

	double ScaleFactor = 1.0 - ShrinkPercentage / 100.0;
	ensure(ScaleFactor != 0.0);

	int32 NumBones = TransformToConvexIndices.Num();
	check(Parents.Num() == NumBones);

	auto SkipBone = [&SimulationType, SkipType](int32 Bone) -> bool
	{
		return SimulationType[Bone] == SkipType;
	};

	auto OnlyConvex = [&TransformToConvexIndices](int32 Bone) -> int32
	{
		ensure(TransformToConvexIndices[Bone].Num() <= 1);
		if (auto It = TransformToConvexIndices[Bone].CreateConstIterator(); It)
		{
			const int32 ConvexIndex = *It;
			return ConvexIndex;
		}
		return INDEX_NONE;
	};

	TArray<TSet<int32>> LeafProximity;
	LeafProximity.SetNum(NumBones);
	if (ensure(GeoProximity))
	{
		for (int32 GeomIdx = 0; GeomIdx < GeoProximity->Num(); GeomIdx++)
		{
			int32 TransformIdx = GeometryToTransformIndex[GeomIdx];
			for (int32 NbrGeomIdx : (*GeoProximity)[GeomIdx])
			{
				LeafProximity[TransformIdx].Add(GeometryToTransformIndex[NbrGeomIdx]);
			}
		}
	}

	auto IsColliding = [&Convexes](int32 ConvexA, int32 ConvexB)
	{
		if (ConvexA == -1 || ConvexB == -1 || Convexes[ConvexA]->NumVertices() == 0 || Convexes[ConvexB]->NumVertices() == 0)
		{
			// at least one of the convex hulls was empty, so cannot be colliding
			return false;
		}
		const Chaos::TRigidTransform<Chaos::FReal, 3> IdentityTransform = Chaos::TRigidTransform<Chaos::FReal, 3>::Identity;
		return GJKIntersection(*Convexes[ConvexA], *Convexes[ConvexB], IdentityTransform);
	};

	auto IsBoneColliding = [&TransformToConvexIndices, &Convexes, &IsColliding](int32 BoneA, int32 BoneB)
	{
		for (int32 ConvexA : TransformToConvexIndices[BoneA])
		{
			for (int32 ConvexB : TransformToConvexIndices[BoneB])
			{
				if (IsColliding(ConvexA, ConvexB))
				{
					return true;
				}
			}
		}
		return false;
	};

	auto GetConvexSpan = [](const Chaos::FConvex& Convex, const Chaos::FVec3& Center, const Chaos::FVec3& Normal) -> Chaos::FVec2
	{
		int32 NumVertices = Convex.NumVertices();
		if (NumVertices == 0)
		{
			return Chaos::FVec2(0, 0);
		}
		Chaos::FReal AlongFirst = (Convex.GetVertex(0) - Center).Dot(FVector3f(Normal));
		Chaos::FVec2 Range(AlongFirst, AlongFirst);
		for (int Idx = 1; Idx < NumVertices; Idx++)
		{
			const float Along = static_cast<float>((Convex.GetVertex(Idx) - Center).Dot(FVector3f(Normal)));
			if (Along < Range.X)
			{
				Range.X = Along;
			}
			else if (Along > Range.Y)
			{
				Range.Y = Along;
			}
		}
		return Range;
	};

	// Score separating plane direction based on how well it separates (lower is better)
	//  and also compute new center for plane + normal. Normal is either the input normal or flipped.
	auto ScoreCutPlane = [&GetConvexSpan](
		const Chaos::FConvex& A, const Chaos::FConvex& B,
		const FChaosPlane& Plane, bool bOneSidedCut,
		Chaos::FVec3& OutCenter, Chaos::FVec3& OutNormal) -> Chaos::FReal
	{
		bool bRangeAValid = false, bRangeBValid = false;
		Chaos::FVec2 RangeA = GetConvexSpan(A, Plane.X(), Plane.Normal());
		Chaos::FVec2 RangeB = GetConvexSpan(B, Plane.X(), Plane.Normal());
		Chaos::FVec2 Union(FMath::Min(RangeA.X, RangeB.X), FMath::Max(RangeA.Y, RangeB.Y));
		// no intersection -- cut plane is separating, this is ideal!
		if (RangeA.X > RangeB.Y || RangeA.Y < RangeB.X)
		{
			if (RangeA.X > RangeB.Y)
			{
				OutCenter = Plane.X() + Plane.Normal() * ((RangeA.X + RangeB.Y) * (Chaos::FReal).5);
				OutNormal = -Plane.Normal();
			}
			else
			{
				OutCenter = Plane.X() + Plane.Normal() * ((RangeA.Y + RangeB.X) * (Chaos::FReal).5);
				OutNormal = Plane.Normal();
			}
			return 0;
		}
		// there was an intersection; find the actual mid plane-center and score it
		Chaos::FVec2 Intersection(FMath::Max(RangeA.X, RangeB.X), FMath::Min(RangeA.Y, RangeB.Y));
		Chaos::FReal IntersectionMid = (Intersection.X + Intersection.Y) * (Chaos::FReal).5;

		// Decide which side of the plane is kept/removed
		Chaos::FVec2 BiggerRange = RangeA;
		Chaos::FReal Sign = 1;
		if (RangeA.Y - RangeA.X < RangeB.Y - RangeB.X)
		{
			BiggerRange = RangeB;
			Sign = -1;
		}
		if (IntersectionMid - BiggerRange.X < BiggerRange.Y - IntersectionMid)
		{
			Sign *= -1;
		}
		OutNormal = Sign * Plane.Normal();

		Chaos::FReal IntersectionCut = IntersectionMid;
		if (bOneSidedCut) // if cut is one-sided, move the plane to the far end of Convex B (which it should not cut)
		{
			// which end depends on which way the output cut plane is oriented
			if (Sign > 0)
			{
				IntersectionCut = RangeB.X;
			}
			else
			{
				IntersectionCut = RangeB.Y;
			}
		}
		OutCenter = Plane.X() + Plane.Normal() * IntersectionCut;

		// Simple score favors small intersection span relative to union span
		// TODO: consider other metrics; e.g. something more directly ~ percent cut away
		return (Intersection.Y - Intersection.X) / (Union.Y - Union.X);
	};

	// Search cut plane options for the most promising one -- 
	// Usually GJK gives a good cut plane, but it can fail badly so we also test a simple 'difference between centers' plane
	// TODO: consider adding more plane options to the search
	auto FindCutPlane = [&ScoreCutPlane](const Chaos::FConvex& A, const Chaos::FConvex& B,
		Chaos::FVec3 CloseA, Chaos::FVec3 CloseB, Chaos::FVec3 Normal,
		bool bOneSidedCut,
		Chaos::FVec3& OutCenter, Chaos::FVec3& OutNormal) -> bool
	{
		OutCenter = (CloseA + CloseB) * .5;
		OutNormal = Normal;
		Chaos::FReal BestScore = ScoreCutPlane(A, B, FChaosPlane(OutCenter, OutNormal), bOneSidedCut, OutCenter, OutNormal);
		Chaos::FVec3 MassSepNormal = (B.GetCenterOfMass() - A.GetCenterOfMass());
		if (MassSepNormal.Normalize() && BestScore > 0)
		{
			Chaos::FVec3 MassSepCenter = (A.GetCenterOfMass() + B.GetCenterOfMass()) * .5;
			Chaos::FReal ScoreB = ScoreCutPlane(A, B,
				FChaosPlane(MassSepCenter, MassSepNormal), bOneSidedCut, MassSepCenter, MassSepNormal);
			if (ScoreB < BestScore)
			{
				BestScore = ScoreB;
				OutCenter = MassSepCenter;
				OutNormal = MassSepNormal;
			}
		}
		if (BestScore == 0)
		{
			return false;
		}
		return true;
	};

	auto FixCollisionWithCut = [&Convexes, &FindCutPlane, &SimplificationDistanceThreshold, &ScaleFactor](int32 ConvexA, int32 ConvexB)
	{
		if (Convexes[ConvexA]->NumVertices() == 0 || Convexes[ConvexB]->NumVertices() == 0)
		{
			// at least one of the convex hulls was empty, so cannot be colliding
			return false;
		}
		Chaos::FReal Depth;
		Chaos::FVec3 CloseA, CloseB, Normal;
		int32 OutIdxA, OutIdxB;
		const Chaos::TRigidTransform<Chaos::FReal, 3> IdentityTransform = Chaos::TRigidTransform<Chaos::FReal, 3>::Identity;
		bool bCollide = Chaos::GJKPenetration(*Convexes[ConvexA], *Convexes[ConvexB], IdentityTransform, Depth, CloseA, CloseB, Normal, OutIdxA, OutIdxB);
		if (bCollide)
		{
			Chaos::FVec3 IdealCenter, IdealNormal;
			if (!FindCutPlane(*Convexes[ConvexA], *Convexes[ConvexB], CloseA, CloseB, Normal, false, IdealCenter, IdealNormal))
			{
				return false;
			}
			FChaosPlane CutPlane(IdealCenter, IdealNormal);
			TArray<Chaos::FConvex::FVec3Type> CutHullPts;
			if (CutHull(*Convexes[ConvexA], CutPlane, true, CutHullPts))
			{
				*Convexes[ConvexA] = MakeHull(CutHullPts, SimplificationDistanceThreshold * ScaleFactor);
			}
			CutHullPts.Reset();
			if (CutHull(*Convexes[ConvexB], CutPlane, false, CutHullPts))
			{
				*Convexes[ConvexB] = MakeHull(CutHullPts, SimplificationDistanceThreshold * ScaleFactor);
			}
		}
		return bCollide;
	};

	// Initialize Children and Depths of tree
	// Fix collisions between input hulls using the input proximity relationships

	int32 MaxDepth = 0;
	TArray<int32> Depths;
	TArray<TArray<int32>> Children;
	Children.SetNum(NumBones);
	Depths.SetNumZeroed(NumBones);
	for (int HullIdx = 0; HullIdx < NumBones; HullIdx++)
	{
		if (SimulationType[HullIdx] == SkipType) // Skip any 'SkipType' elements (generally embedded geometry)
		{
			Depths[HullIdx] = -1;
			continue;
		}
		if (Parents[HullIdx] != INDEX_NONE)
		{
			if (SimulationType[Parents[HullIdx]] != LeafType)
			{
				Children[Parents[HullIdx]].Add(HullIdx);
			}
			else
			{
				Depths[HullIdx] = -1; // child-of-leaf == embedded geometry, just ignore it
				continue;
			}
		}
		int32 Depth = 0, WalkParent = HullIdx;
		while (Parents[WalkParent] != INDEX_NONE)
		{
			Depth++;
			WalkParent = Parents[WalkParent];
		}
		Depths[HullIdx] = Depth;
		MaxDepth = FMath::Max(Depth, MaxDepth);

		if (bRemoveOverlaps && bRemoveLeafOverlaps && !HasCustomConvexFn(HullIdx) && TransformToConvexIndices[HullIdx].Num() > 0)
		{
			const TSet<int32>& Neighbors = LeafProximity[HullIdx];
			for (int32 NbrIdx : Neighbors)
			{
				if (!HasCustomConvexFn(NbrIdx) && NbrIdx < HullIdx && TransformToConvexIndices[NbrIdx].Num() > 0)
				{
					// TODO: consider a one-sided cut if one of the bones has custom convexes
					FixCollisionWithCut(OnlyConvex(HullIdx), OnlyConvex(NbrIdx));
				}
			}
		}
	}

	TArray<int32> ByDepthOrder;
	ByDepthOrder.Reserve(NumBones);
	for (int32 ProcessDepth = MaxDepth; ProcessDepth >= 0; --ProcessDepth)
	{
		for (int32 Bone = 0; Bone < NumBones; Bone++)
		{
			if (Depths[Bone] == ProcessDepth)
			{
				ByDepthOrder.Add(Bone);
			}
		}
	}

	auto AddLeaves = [&Children, &SimulationType, LeafType](int32 Bone, TArray<int32>& Leaves)
	{
		TArray<int32> ToExpand;
		ToExpand.Add(Bone);
		while (ToExpand.Num() > 0)
		{
			int32 ToProcess = ToExpand.Pop(EAllowShrinking::No);
			if (SimulationType[ToProcess] == LeafType)
			{
				Leaves.Add(ToProcess);
			}
			else if (Children[ToProcess].Num() > 0)
			{
				ToExpand.Append(Children[ToProcess]);
			}
		}
	};

	// Fill OutChildren with the shallowest descendents of Bone that have convex(es) (i.e., the direct children if they all have convexes, otherwise descend further to grandchildren searching for convex hulls)
	auto AddDescendentsWithHulls = [&Children, &TransformToConvexIndices, &SimulationType, LeafType](int32 Bone, TArray<int32>& OutChildren)
	{
		TArray<int32> ToExpand = Children[Bone];
		while (ToExpand.Num() > 0)
		{
			int32 ToProcess = ToExpand.Pop(EAllowShrinking::No);
			if (TransformToConvexIndices[ToProcess].Num() > 0)
			{
				OutChildren.Add(ToProcess);
			}
			else if (Children[ToProcess].Num() > 0)
			{
				ToExpand.Append(Children[ToProcess]);
			}
		}
	};

	// Use initial leaf proximity to compute which cluster bones are in proximity to which neighbors at the same level of the bone hierarchy

	TArray<TSet<int32>> SameDepthClusterProximity;
	if (bRemoveOverlaps)
	{
		SameDepthClusterProximity.SetNum(NumBones);
		for (int32 Bone = 0; Bone < NumBones; Bone++)
		{
			for (int32 NbrBone : LeafProximity[Bone])
			{
				if (Bone > NbrBone)
				{
					continue;
				}

				auto FindCommonParent = [&Parents](int32 BoneA, int32 BoneB)
				{
					int AParent = Parents[BoneA];
					int BParent = Parents[BoneB];
					if (AParent == BParent) // early out if they're in the same cluster
					{
						return AParent;
					}

					TSet<int32> AParents;
					while (AParent != INDEX_NONE)
					{
						AParents.Add(AParent);
						AParent = Parents[AParent];
					}

					while (BParent != INDEX_NONE && !AParents.Contains(BParent))
					{
						BParent = Parents[BParent];
					}
					return BParent;
				};
				int32 CommonParent = FindCommonParent(Bone, NbrBone);

				auto ConnectAtMatchingDepth = [&SameDepthClusterProximity, &Parents, CommonParent, &Depths](int32 BoneA, int32 BoneToWalkParents)
				{
					if (BoneA == INDEX_NONE || BoneToWalkParents == INDEX_NONE)
					{
						return;
					}

					int32 DepthA = Depths[BoneA];
					int32 DepthB = Depths[BoneToWalkParents];
					while (BoneToWalkParents != INDEX_NONE && BoneToWalkParents != CommonParent && DepthB >= DepthA)
					{
						if (DepthB == DepthA)
						{
							bool bWasInSet = false;
							SameDepthClusterProximity[BoneToWalkParents].Add(BoneA, &bWasInSet);
							if (bWasInSet)
							{
								break;
							}
							SameDepthClusterProximity[BoneA].Add(BoneToWalkParents);
						}

						BoneToWalkParents = Parents[BoneToWalkParents];
						DepthB--;
					}
				};

				// connect bone to neighboring bone's clusters
				ConnectAtMatchingDepth(Bone, Parents[NbrBone]);

				// walk chain of parents of bone and connect each to the chain of neighboring bone + its parents
				int32 BoneParent = Parents[Bone];
				while (BoneParent != INDEX_NONE && BoneParent != CommonParent)
				{
					ConnectAtMatchingDepth(BoneParent, NbrBone);
					BoneParent = Parents[BoneParent];
				}
			}
		}
	}

	// Compute initial hulls at all levels and filter out any that are too large relative to the geometry they contain

	TArray<TSet<int32>> ClusterProximity;
	if (bRemoveOverlaps)
	{
		ClusterProximity.SetNum(NumBones);
	}
	TArrayView<int32> DepthOrderView(ByDepthOrder);

	for (int32 Idx = 0, SliceEndIdx = Idx + 1; Idx < ByDepthOrder.Num(); Idx = SliceEndIdx)
	{
		FCriticalSection ConvexesCS;

		int32 ProcessDepth = Depths[ByDepthOrder[Idx]];
		for (SliceEndIdx = Idx + 1; SliceEndIdx < ByDepthOrder.Num(); SliceEndIdx++)
		{
			if (Depths[ByDepthOrder[SliceEndIdx]] != ProcessDepth)
			{
				break;
			}
		}
		TArrayView<int32> SameDepthSlice = DepthOrderView.Slice(Idx, SliceEndIdx - Idx);
		
		ParallelFor(SameDepthSlice.Num(), [&](int32 SliceIdx)
		{
			int32 Bone = SameDepthSlice[SliceIdx];
			checkSlow(Depths[Bone] == ProcessDepth);
			if (!SkipBone(Bone))
			{
				if (TransformToConvexIndices[Bone].Num() == 0 && !HasCustomConvexFn(Bone))
				{
					TArray<Chaos::FConvex::FVec3Type> JoinedHullPts;
					TArray<int32> ChildrenWithHulls;
					AddDescendentsWithHulls(Bone, ChildrenWithHulls);
					for (int32 Child : ChildrenWithHulls)
					{
						for (int32 ConvexIdx : TransformToConvexIndices[Child])
						{
							FVector Pivot;
							ConvexesCS.Lock();
							Chaos::FConvex* Convex = Convexes[ConvexIdx].GetReference();
							Pivot = ConvexPivots[ConvexIdx];
							ConvexesCS.Unlock();
							if (ShrinkPercentage != 0.0)
							{
								AddUnscaledPts(JoinedHullPts, Convex->GetVertices(), Pivot, ShrinkPercentage);
							}
							else
							{
								JoinedHullPts.Append(Convex->GetVertices());
							}
						}
					}
					
					if (JoinedHullPts.Num() > 0)
					{
						FilterHullPoints(JoinedHullPts, SimplificationDistanceThreshold);
						FVector Pivot = ScaleHullPoints(JoinedHullPts, ShrinkPercentage);
						Chaos::FConvex* Hull = new Chaos::FConvex(JoinedHullPts, UE_KINDA_SMALL_NUMBER);
						Chaos::FConvexPtr HullPtr(Hull);
						bool bIsTooBig = false;
						if (Volume)
						{
							Chaos::FReal HullVolume = Hull->GetVolume();
							if (HullVolume > Chaos::FReal((*Volume)[Bone]) * (1.0 + CanExceedFraction))
							{
								bIsTooBig = true;
							}
						}
						if (!bIsTooBig)
						{
							ConvexesCS.Lock();
							int32 ConvexIdx = Convexes.Add(MoveTemp(HullPtr));
							AddPivot(Convexes, ConvexPivots, Pivot);
							ConvexesCS.Unlock();
							TransformToConvexIndices[Bone].Add(ConvexIdx);
						}
					}
				}
			}
		});

		if (!bRemoveOverlaps)
		{
			continue;
		}

		// Compute cluster proximity
		for (int32 BoneA : SameDepthSlice)
		{
			if (SkipBone(BoneA) || TransformToConvexIndices[BoneA].Num() == 0)
			{
				continue;
			}
			for (int32 BoneB : SameDepthClusterProximity[BoneA])
			{
				if (BoneB < BoneA && IsBoneColliding(BoneA, BoneB))
				{
					int32 Bones[2]{ BoneA, BoneB };
					for (int32 BoneIdx = 0; BoneIdx < 2; BoneIdx++)
					{
						int32 ParentBone = Bones[BoneIdx];
						int32 OtherBone = Bones[1 - BoneIdx];
						TArray<int32> TraverseBones;
						TraverseBones.Append(Children[ParentBone]);
						while (TraverseBones.Num() > 0)
						{
							int32 ToProc = TraverseBones.Pop(EAllowShrinking::No);
							if (IsBoneColliding(OtherBone, ToProc))
							{
								ClusterProximity[OtherBone].Add(ToProc);
								ClusterProximity[ToProc].Add(OtherBone);
								TraverseBones.Append(Children[ToProc]);
							}
						}
					}

					ClusterProximity[BoneA].Add(BoneB);
					ClusterProximity[BoneB].Add(BoneA);
				}
			}
		}
	}

	if (!bRemoveOverlaps)
	{
		return; // rest of function is just for removing overlaps
	}


	// Compute all initial non-leaf hull volumes

	TArray<double> NonLeafVolumes; // Original volumes of non-leaf hulls (to compare against progressively cut-down volume as intersections are removed)
	NonLeafVolumes.SetNumZeroed(Convexes.Num());
	ParallelFor(NumBones, [&](int32 Bone)
	{
		bool bCustom = HasCustomConvexFn(Bone);
		if (!bCustom && // if we need an automatic hull
			Children[Bone].Num() > 0 && !SkipBone(Bone) && // and we have children && aren't embedded geo
			TransformToConvexIndices[Bone].Num() == 1 // and hull wasn't already ruled out by CanExceedFraction
			)
		{
			int32 ConvexIdx = OnlyConvex(Bone);
			checkSlow(ConvexIdx > -1); // safe to assume because the if-statement verified "TransformToConvexIndices[Bone].Num() == 1" 
			NonLeafVolumes[ConvexIdx] = Convexes[ConvexIdx]->GetVolume();
		}
	});

	// if bOneSidedCut, then only ConvexA is cut; ConvexB is left unchanged
	auto CutIfOk = [&Convexes, &NonLeafVolumes, &FindCutPlane, &FracAllowRemove, &SimplificationDistanceThreshold, &ScaleFactor](bool bOneSidedCut, int32 ConvexA, int32 ConvexB) -> bool
	{
		Chaos::FReal Depth;
		Chaos::FVec3 CloseA, CloseB, Normal;
		int32 OutIdxA, OutIdxB;
		const Chaos::TRigidTransform<Chaos::FReal, 3> IdentityTransform = Chaos::TRigidTransform<Chaos::FReal, 3>::Identity;
		bool bCollide = Chaos::GJKPenetration(*Convexes[ConvexA], *Convexes[ConvexB], IdentityTransform, Depth, CloseA, CloseB, Normal, OutIdxA, OutIdxB);
		if (bCollide)
		{
			Chaos::FVec3 IdealCenter, IdealNormal;
			FindCutPlane(*Convexes[ConvexA], *Convexes[ConvexB], CloseA, CloseB, Normal, bOneSidedCut, IdealCenter, IdealNormal);
			FChaosPlane CutPlane(IdealCenter, IdealNormal);

			// Tentatively create the clipped hulls
			Chaos::FConvex CutHullA, CutHullB;
			bool bCreatedA = false, bCreatedB = false;
			TArray<Chaos::FConvex::FVec3Type> CutHullPts;
			if (CutHull(*Convexes[ConvexA], CutPlane, true, CutHullPts))
			{
				if (CutHullPts.Num() < 4) // immediate reject zero-volume results
				{
					return false;
				}
				CutHullA = MakeHull(CutHullPts, SimplificationDistanceThreshold * ScaleFactor);
				bCreatedA = true;
			}
			if (!bOneSidedCut)
			{
				CutHullPts.Reset();
				if (CutHull(*Convexes[ConvexB], CutPlane, false, CutHullPts))
				{
					CutHullB = MakeHull(CutHullPts, SimplificationDistanceThreshold * ScaleFactor);
					bCreatedB = true;
				}
			}

			// Test if the clipped hulls have become too small vs the original volumes
			if (ensure(ConvexA < NonLeafVolumes.Num()) && NonLeafVolumes[ConvexA] > 0 && bCreatedA && CutHullA.GetVolume() / NonLeafVolumes[ConvexA] < 1 - FracAllowRemove)
			{
				return false;
			}
			if (!bOneSidedCut)
			{
				if (ensure(ConvexB < NonLeafVolumes.Num()) && NonLeafVolumes[ConvexB] > 0 && bCreatedB && CutHullB.GetVolume() / NonLeafVolumes[ConvexB] < 1 - FracAllowRemove)
				{
					return false;
				}
			}

			// If the clipped hulls were large enough, go ahead and set them as the new hulls
			if (bCreatedA)
			{
				*Convexes[ConvexA] = MoveTemp(CutHullA);
			}
			if (bCreatedB)
			{
				*Convexes[ConvexB] = MoveTemp(CutHullB);
			}

			return true;
		}
		else
		{
			return true; // no cut needed, so was ok
		}
	};

	// re-process all non-leaf bones
	for (int32 Idx = 0, SliceEndIdx = Idx + 1; Idx < DepthOrderView.Num(); Idx = SliceEndIdx)
	{
		int32 ProcessDepth = Depths[DepthOrderView[Idx]];
		for (SliceEndIdx = Idx + 1; SliceEndIdx < DepthOrderView.Num(); SliceEndIdx++)
		{
			if (Depths[DepthOrderView[SliceEndIdx]] != ProcessDepth)
			{
				break;
			}
		}
		TArrayView<int32> SameDepthSlice = DepthOrderView.Slice(Idx, SliceEndIdx - Idx);

		TSet<int32> WasNotOk;

		for (int32 Bone : SameDepthSlice)
		{
			bool bCustom = HasCustomConvexFn(Bone);
			if (bCustom || Children[Bone].Num() == 0 || TransformToConvexIndices[Bone].Num() == 0 || WasNotOk.Contains(Bone))
			{
				continue;
			}
			for (int32 Nbr : ClusterProximity[Bone])
			{
				if (WasNotOk.Contains(Nbr) || TransformToConvexIndices[Nbr].Num() == 0)
				{
					continue;
				}
				bool bNbrCustom = HasCustomConvexFn(Nbr);

				// if the neighbor is less deep and not a leaf, skip processing this to favor processing the neighbor instead
				if (Depths[Bone] > Depths[Nbr] && Children[Nbr].Num() > 0)
				{
					continue;
				}
				// If we only consider cluster-vs-cluster overlap, and the neighbor is a leaf, do not consider it
				if (OverlapRemovalMethod == EConvexOverlapRemoval::OnlyClustersVsClusters && Children[Nbr].Num() == 0)
				{
					continue;
				}

				bool bAllOk = true;
				for (int32 ConvexBone : TransformToConvexIndices[Bone])
				{
					for (int32 ConvexNbr : TransformToConvexIndices[Nbr])
					{
						bool bOneSidedCut = Depths[Bone] != Depths[Nbr] || Children[Nbr].Num() == 0 || bNbrCustom;

						bool bWasOk = CutIfOk(bOneSidedCut, ConvexBone, ConvexNbr);

						// cut would have removed too much; just fall back to using the hulls of children
						// TODO: attempt splitting hulls before fully falling back to this
						if (!bWasOk)
						{
							bAllOk = false;
							auto ResetBoneHulls = [&WasNotOk, &TransformToConvexIndices, &Convexes](int32 ToReset)
							{
								WasNotOk.Add(ToReset);
								for (int32 ConvexIdx : TransformToConvexIndices[ToReset])
								{
									// we just leave these hulls in as null, without any references here
									// and come through and clear them later after everything is set up in the geometry collection
									// (because that has the built-in machinery to update the TransformToConvexIndices indices accordingly)
									Convexes[ConvexIdx].SafeRelease();
								}
								TransformToConvexIndices[ToReset].Reset();
							};
							ResetBoneHulls(Bone);
							if (!bOneSidedCut)
							{
								ResetBoneHulls(Nbr);
							}
							break;
						}
					}
					if (!bAllOk)
					{
						break;
					}
				}
			}
		}
	}

	return;
}


// helper to compute the volume of an individual piece of geometry
double ComputeGeometryVolume(
	const FManagedArrayCollection* Collection,
	int32 GeometryIdx,
	const FTransform& GlobalTransform,
	double ScalePerDimension
)
{
	GeometryCollection::Facades::FCollectionMeshFacade MeshFacade(*Collection);
	const TManagedArray<int32>& VertexStart = MeshFacade.VertexStartAttribute.Get();
	const TManagedArray<int32>& VertexCount = MeshFacade.VertexCountAttribute.Get();
	const TManagedArray<FVector3f>& Vertex = MeshFacade.VertexAttribute.Get();
	const TManagedArray<int32>& FaceStart = MeshFacade.FaceStartAttribute.Get();
	const TManagedArray<int32>& FaceCount = MeshFacade.FaceCountAttribute.Get();
	const TManagedArray<FIntVector>& Indices = MeshFacade.IndicesAttribute.Get();
	int32 VStart = VertexStart[GeometryIdx];
	int32 VEnd = VStart + VertexCount[GeometryIdx];
	if (VStart == VEnd)
	{
		return 0.0;
	}
	FVector3d Center = FVector::ZeroVector;
	for (int32 VIdx = VStart; VIdx < VEnd; VIdx++)
	{
		FVector Pos = GlobalTransform.TransformPosition((FVector)Vertex[VIdx]);
		Center += (FVector3d)Pos;
	}
	Center /= double(VEnd - VStart);
	int32 FStart = FaceStart[GeometryIdx];
	int32 FEnd = FStart + FaceCount[GeometryIdx];
	double VolOut = 0;
	for (int32 FIdx = FStart; FIdx < FEnd; FIdx++)
	{
		FIntVector Tri = Indices[FIdx];
		FVector3d V0 = (FVector3d)GlobalTransform.TransformPosition((FVector)Vertex[Tri.X]);
		FVector3d V1 = (FVector3d)GlobalTransform.TransformPosition((FVector)Vertex[Tri.Y]);
		FVector3d V2 = (FVector3d)GlobalTransform.TransformPosition((FVector)Vertex[Tri.Z]);

		// add volume of the tetrahedron formed by the triangles and the reference point
		FVector3d V1mRef = (V1 - Center) * ScalePerDimension;
		FVector3d V2mRef = (V2 - Center) * ScalePerDimension;
		FVector3d N = V2mRef.Cross(V1mRef);

		VolOut += ((V0 - Center) * ScalePerDimension).Dot(N) / 6.0;
	}
	return VolOut;
}

// Helper to append the triangles of a convex hull to a dynamic mesh
static void AddConvexHullToCompactDynamicMesh(const ::Chaos::FConvex* InConvexHull, UE::Geometry::FDynamicMesh3& Mesh, const FTransform* OptionalTransform = nullptr, bool bInvertFaces = true)
{
	check(Mesh.IsCompact());

	const ::Chaos::FConvexStructureData& ConvexStructure = InConvexHull->GetStructureData();
	const int32 NumV = InConvexHull->NumVertices();
	const int32 NumP = InConvexHull->NumPlanes();
	int32 StartV = Mesh.MaxVertexID();
	for (int32 VIdx = 0; VIdx < NumV; ++VIdx)
	{
		FVector3d V = (FVector3d)InConvexHull->GetVertex(VIdx);
		if (OptionalTransform)
		{
			V = OptionalTransform->TransformPosition(V);
		}
		int32 MeshVIdx = Mesh.AppendVertex(V);
		checkSlow(MeshVIdx == VIdx + StartV); // Must be true because the mesh is compact
	}
	for (int32 PIdx = 0; PIdx < NumP; ++PIdx)
	{
		const int32 NumFaceV = ConvexStructure.NumPlaneVertices(PIdx);
		const int32 V0 = StartV + ConvexStructure.GetPlaneVertex(PIdx, 0);
		for (int32 SubIdx = 1; SubIdx + 1 < NumFaceV; ++SubIdx)
		{
			int32 V1 = StartV + ConvexStructure.GetPlaneVertex(PIdx, SubIdx);
			int32 V2 = StartV + ConvexStructure.GetPlaneVertex(PIdx, SubIdx + 1);
			if (bInvertFaces)
			{
				Swap(V1, V2);
			}
			int32 ResultTID = Mesh.AppendTriangle(UE::Geometry::FIndex3i(V0, V1, V2));
			constexpr bool bFixNonmanifoldWithDuplicates = true;
			if (bFixNonmanifoldWithDuplicates && ResultTID == UE::Geometry::FDynamicMesh3::NonManifoldID)
			{
				// failed to append due to a non-manifold triangle; try adding all the vertices independently so we at least capture the shape
				// note: this should not happen for normal convex hulls, but the current convex hull algorithm does some aggressive face merging that sometimes creates weird geometry
				UE::Geometry::FIndex3i DuplicateVerts(
					Mesh.AppendVertex(Mesh.GetVertex(V0)),
					Mesh.AppendVertex(Mesh.GetVertex(V1)),
					Mesh.AppendVertex(Mesh.GetVertex(V2))
				);
				Mesh.AppendTriangle(DuplicateVerts);
			}
		}
	}
}

// Helper to convert a geometry collection's geometry to a dynamic mesh
static UE::Geometry::FDynamicMesh3 GeometryToDynamicMesh(const FGeometryCollection& Geometry, int32 GeomIdx, FTransform* OptionalTransform)
{
	UE::Geometry::FDynamicMesh3 Mesh;

	int32 VStart = Geometry.VertexStart[GeomIdx];
	int32 VCount = Geometry.VertexCount[GeomIdx];
	int32 VEnd = VStart + VCount;

	for (int32 Idx = VStart; Idx < VEnd; ++Idx)
	{
		FVector3d V = (FVector3d)Geometry.Vertex[Idx];
		if (OptionalTransform)
		{
			V = OptionalTransform->TransformPosition(V);
		}
		Mesh.AppendVertex(V);
	}

	int32 FStart = Geometry.FaceStart[GeomIdx];
	int32 FCount = Geometry.FaceCount[GeomIdx];
	int32 FEnd = FStart + FCount;
	for (int32 Idx = FStart; Idx < FEnd; ++Idx)
	{
		FIntVector Face = Geometry.Indices[Idx];
		Mesh.AppendTriangle(Face.X - VStart, Face.Y - VStart, Face.Z - VStart);
	}

	return Mesh;
}

/// Helper to get convex hulls from a geometry collection in the format required by CreateNonoverlappingConvexHulls
void HullsFromGeometry(
	FGeometryCollection& Geometry,
	const TArray<FTransform>& GlobalTransformArray,
	TFunctionRef<bool(int32)> HasCustomConvexFn,
	TArray<Chaos::FConvexPtr>& Convexes,
	TArray<FVector>& ConvexPivots,
	TArray<TSet<int32>>& TransformToConvexIndices,
	const TManagedArray<int32>& SimulationType,
	int32 RigidType,
	double SimplificationDistanceThreshold,
	double OverlapRemovalShrinkPercent,
	TFunction<bool(int32)> SkipBoneFn = nullptr,
	const FGeometryCollectionConvexUtility::FConvexDecompositionSettings* OptionalDecompositionSettings = nullptr,
	const TArray<FGeometryCollectionConvexUtility::FTransformedConvex>* OptionalIntersectConvexHulls = nullptr,
	const TArray<TSet<int32>>* OptionalTransformToIntersectHulls = nullptr,
	TArray<FGeometryCollectionConvexUtility::FSphereCoveringInfo>* OutComputedNavigableSpheres = nullptr
)
{
	TArray<FVector> GlobalVertices;

	TOptional<FGeometryCollectionConvexUtility::FGeometryCollectionConvexData> OrigConvexData = FGeometryCollectionConvexUtility::GetConvexHullDataIfPresent(&Geometry);
	
	GlobalVertices.SetNum(Geometry.Vertex.Num());
	for (int32 Idx = 0; Idx < GlobalVertices.Num(); Idx++)
	{
		GlobalVertices[Idx] = GlobalTransformArray[Geometry.BoneMap[Idx]].TransformPosition(FVector(Geometry.Vertex[Idx]));
	}

	double ScaleFactor = 1 - OverlapRemovalShrinkPercent / 100.0;

	int32 NumBones = Geometry.TransformToGeometryIndex.Num();
	TransformToConvexIndices.SetNum(NumBones);
	FCriticalSection HullCS;
	FCriticalSection NavSpheresCS;
	ParallelFor(NumBones, [&](int32 Idx)
	{
		if (SkipBoneFn && !SkipBoneFn(Idx))
		{
			return;
		}

		int32 GeomIdx = Geometry.TransformToGeometryIndex[Idx];
		if (OrigConvexData.IsSet() && HasCustomConvexFn(Idx))
		{
			// custom convex hulls are kept, but transformed to global space + copied to the new data structure
			const FTransform& Transform = GlobalTransformArray[Idx];
			for (int32 OrigConvexIdx : OrigConvexData->TransformToConvexIndices[Idx])
			{
				TArray<Chaos::FConvex::FVec3Type> HullPts;
				// As we already have a hull, use the center of mass as the pivot
				FVector COM = (FVector)OrigConvexData->ConvexHull[OrigConvexIdx]->GetCenterOfMass();
				HullPts.Reserve(OrigConvexData->ConvexHull[OrigConvexIdx]->NumVertices());
				for (const Chaos::FConvex::FVec3Type& P : OrigConvexData->ConvexHull[OrigConvexIdx]->GetVertices())
				{
					HullPts.Add(Chaos::FConvex::FVec3Type(Transform.TransformPosition( // transform to the global space
						(FVector(P) - COM) * ScaleFactor + COM // scale in the original coordinate frame
						)));
				}
				// Do not simplify hulls when we're just trying to transform them
				Chaos::FConvexPtr Hull( new Chaos::FConvex(HullPts, UE_KINDA_SMALL_NUMBER));
				HullCS.Lock();
				int32 NewConvexIdx = Convexes.Add(MoveTemp(Hull));
				AddPivot(Convexes, ConvexPivots, COM);
				HullCS.Unlock();
				TransformToConvexIndices[Idx].Add(NewConvexIdx);
			}
		}
		else if (SimulationType[Idx] == RigidType && GeomIdx != INDEX_NONE)
		{
			// If external intersection is requested, do the merge + intersection for the given bone here so it can be used by ComputeHull and/or the decomp
			bool bHasExternal = OptionalIntersectConvexHulls && OptionalTransformToIntersectHulls;
			TUniquePtr<UE::Geometry::FDynamicMesh3> GeometryIntersectedWithExternalHull = nullptr;
			if (bHasExternal)
			{
				const TSet<int32>& HullInds = (*OptionalTransformToIntersectHulls)[Idx];
				UE::Geometry::FDynamicMesh3 MergedHullMesh;
				for (int32 HullIdx : HullInds)
				{
					// Append transformed external hull mesh
					AddConvexHullToCompactDynamicMesh((*OptionalIntersectConvexHulls)[HullIdx].Convex.GetReference(), MergedHullMesh, &(*OptionalIntersectConvexHulls)[HullIdx].Transform);
				}
				if (HullInds.Num() > 1)
				{
					// Self-union the merged hull mesh -- mainly needed to help ensure the intersection volume accuracy
					// i.e., in the (bAttemptDecomposition && OptionalDecompositionSettings->MaxGeoToHullVolumeRatioToDecompose < 1.0) case below
					UE::Geometry::FMeshSelfUnion Union(&MergedHullMesh);
					Union.Compute();
				}
				if (MergedHullMesh.TriangleCount() > 0)
				{
					UE::Geometry::FDynamicMesh3 GeometryMesh = GeometryToDynamicMesh(Geometry, GeomIdx, nullptr);
					UE::Geometry::FMeshBoolean Boolean(&GeometryMesh, GlobalTransformArray[Idx], &MergedHullMesh, GlobalTransformArray[Idx],
						&GeometryMesh, UE::Geometry::FMeshBoolean::EBooleanOp::Intersect);

					Boolean.Compute();
					if (GeometryMesh.VertexCount() > 0)
					{
						GeometryIntersectedWithExternalHull = MakeUnique<UE::Geometry::FDynamicMesh3>();
						*GeometryIntersectedWithExternalHull = MoveTemp(GeometryMesh);
					}
				}
			}

			auto ComputeHull = [&Geometry, &GlobalVertices, SimplificationDistanceThreshold, OverlapRemovalShrinkPercent, GeomIdx,
				&GeometryIntersectedWithExternalHull](FVector& PivotOut) -> ::Chaos::FConvexPtr
			{
				TArray<Chaos::FConvex::FVec3Type> HullPts;
				if (GeometryIntersectedWithExternalHull)
				{
					HullPts.Reserve(GeometryIntersectedWithExternalHull->VertexCount());
					for (FVector3d Vertex : GeometryIntersectedWithExternalHull->VerticesItr())
					{
						HullPts.Add((Chaos::FConvex::FVec3Type)Vertex);
					}
				}
				else
				{
					int32 VStart = Geometry.VertexStart[GeomIdx];
					int32 VCount = Geometry.VertexCount[GeomIdx];
					int32 VEnd = VStart + VCount;

					HullPts.Reserve(VCount);
					for (int32 VIdx = VStart; VIdx < VEnd; VIdx++)
					{
						HullPts.Add(GlobalVertices[VIdx]);
					}
				}
				ensure(HullPts.Num() > 0);
				FilterHullPoints(HullPts, SimplificationDistanceThreshold);
				PivotOut = ScaleHullPoints(HullPts, OverlapRemovalShrinkPercent);
				return Chaos::FConvexPtr(new Chaos::FConvex(HullPts, UE_KINDA_SMALL_NUMBER));
			};
			Chaos::FConvexPtr Hull = nullptr;
			FVector HullPivot = FVector::ZeroVector;

			if (OptionalDecompositionSettings)
			{
				bool bAttemptDecomposition = OptionalDecompositionSettings->MaxHullsPerGeometry > 1 || OptionalDecompositionSettings->ErrorTolerance > 0.0 || OptionalDecompositionSettings->bProtectNegativeSpace || OptionalDecompositionSettings->CustomEmptySpace;
				double InitialGeoVolume = 0.0, InitialHullVolume = 0.0;
				if (bAttemptDecomposition && (OptionalDecompositionSettings->MinGeoVolumeToDecompose > 0.0 || OptionalDecompositionSettings->MaxGeoToHullVolumeRatioToDecompose < 1.0))
				{
					if (GeometryIntersectedWithExternalHull)
					{
						InitialGeoVolume = UE::Geometry::TMeshQueries<UE::Geometry::FDynamicMesh3>::GetVolumeNonWatertight(*GeometryIntersectedWithExternalHull);
					}
					else
					{
						InitialGeoVolume = ComputeGeometryVolume(&Geometry, GeomIdx, GlobalTransformArray[Idx], 1.0);
					}
				}
				if (bAttemptDecomposition && InitialGeoVolume < OptionalDecompositionSettings->MinGeoVolumeToDecompose)
				{
					bAttemptDecomposition = false; // too small to consider for decomposition
				}
				if (bAttemptDecomposition && OptionalDecompositionSettings->MaxGeoToHullVolumeRatioToDecompose < 1.0)
				{
					Hull = ComputeHull(HullPivot);
					InitialHullVolume = (double)Hull->GetVolume();
					// Hull was scaled down by the overlap removal shrink percent, so to compute a correct ratio we need to adjust the geo volume in the same way
					double VolumeScale = ScaleFactor * ScaleFactor * ScaleFactor;
					if ((VolumeScale * InitialGeoVolume) / InitialHullVolume >= OptionalDecompositionSettings->MaxGeoToHullVolumeRatioToDecompose)
					{
						bAttemptDecomposition = false;
					}
				}

				if (bAttemptDecomposition)
				{
					UE::Geometry::FConvexDecomposition3 Decomposition;
					UE::Geometry::FConvexDecomposition3::FPreprocessMeshOptions PreprocessOptions;
					PreprocessOptions.bMergeEdges = true;
					PreprocessOptions.ThickenInputAfterHullFailure = OptionalDecompositionSettings->ThickenHullOnFailure;

					if (GeometryIntersectedWithExternalHull)
					{
						Decomposition.InitializeFromMesh(*GeometryIntersectedWithExternalHull, PreprocessOptions);
					}
					else
					{
						int32 VertexStart = Geometry.VertexStart[GeomIdx];
						TArrayView<const FVector3f> VerticesView(Geometry.Vertex.GetData() + VertexStart, Geometry.VertexCount[GeomIdx]);
						TArrayView<const FIntVector3> FacesView(Geometry.Indices.GetData() + Geometry.FaceStart[GeomIdx], Geometry.FaceCount[GeomIdx]);
						Decomposition.InitializeFromIndexMesh(VerticesView, FacesView, PreprocessOptions, -VertexStart);
					}

					const bool bIsSolid = Decomposition.IsInputSolid();
					Decomposition.bTreatAsSolid = bIsSolid;

					if (OptionalDecompositionSettings->bProtectNegativeSpace || OptionalDecompositionSettings->CustomEmptySpace)
					{
						// Use settings tuned for navigation-driven decomposition

						UE::Geometry::FNegativeSpaceSampleSettings Settings;
						Settings.ApplyDefaults();
						Settings.bOnlyConnectedToHull = OptionalDecompositionSettings->bOnlyConnectedToHull;
						Settings.MinRadius = OptionalDecompositionSettings->NegativeSpaceMinRadius;
						Settings.ReduceRadiusMargin = OptionalDecompositionSettings->NegativeSpaceTolerance;
						Settings.bRequireSearchSampleCoverage = true;
						Settings.bAllowSamplesInsideMesh = !bIsSolid;

						Decomposition.MaxConvexEdgePlanes = 4;
						Decomposition.bSplitDisconnectedComponents = false;
						Decomposition.ConvexEdgeAngleMoreSamplesThreshold = 180;
						Decomposition.ThickenAfterHullFailure = PreprocessOptions.ThickenInputAfterHullFailure;

						if (OptionalDecompositionSettings->bProtectNegativeSpace)
						{
							Decomposition.InitializeNegativeSpace(Settings);
						}
						if (OptionalDecompositionSettings->CustomEmptySpace)
						{
							// Get the matrix transform from CustomEmptySpace to bone-local space
							FMatrix ToBoneLocal = OptionalDecompositionSettings->CustomEmptySpaceWorldTransform.ToMatrixWithScale() * GlobalTransformArray[Idx].ToInverseMatrixWithScale();
							// conservatively scale radii in the non-uniform case, since we don't want the spheres to overlap the input geometry
							double ToBoneLocalScale = ToBoneLocal.GetMinimumAxisScale();
							Decomposition.AddCoveringToInitialNegativeSpace(*OptionalDecompositionSettings->CustomEmptySpace, ToBoneLocal, ToBoneLocalScale);
						}

						if (OutComputedNavigableSpheres)
						{
							FGeometryCollectionConvexUtility::FSphereCoveringInfo SphereInfo
							{
								.Transform = GlobalTransformArray[Idx],
								.SourceBoneIndex = Idx,
								.SphereCovering = Decomposition.GetNegativeSpace()
							};
							SphereInfo.SphereCovering.ApplyTransform(Decomposition.ResultTransform);
							NavSpheresCS.Lock();
							OutComputedNavigableSpheres->Add(MoveTemp(SphereInfo));
							NavSpheresCS.Unlock();
						}

						constexpr int32 MaxAllowedSplits = 1000000; // more parts than any expected / reasonable decomposition
						int32 TargetNumSplits = OptionalDecompositionSettings->MaxHullsPerGeometry > 0 ? OptionalDecompositionSettings->MaxHullsPerGeometry - 1 : MaxAllowedSplits + 1;
						for (int32 Split = 0; Split < TargetNumSplits; Split++)
						{
							int32 NumSplit = Decomposition.SplitWorst(false, -1, true, Settings.ReduceRadiusMargin * .5);

							if (NumSplit == 0)
							{
								break;
							}

							if (!ensureMsgf(Split < MaxAllowedSplits, TEXT("Convex decomposition split the input %d times; likely stuck in a loop"), Split))
							{
								break;
							}
						}

						Decomposition.FixHullOverlapsInNegativeSpace();
						int32 TargetNumPieces = OptionalDecompositionSettings->MaxHullsPerGeometry > 0 ? OptionalDecompositionSettings->MaxHullsPerGeometry : -1;
						Decomposition.MergeBest(TargetNumPieces, 0, OptionalDecompositionSettings->MinThicknessTolerance, true);
					}
					else
					{
						Decomposition.Compute(OptionalDecompositionSettings->MaxHullsPerGeometry, OptionalDecompositionSettings->NumAdditionalSplits,
							OptionalDecompositionSettings->ErrorTolerance, OptionalDecompositionSettings->MinThicknessTolerance, OptionalDecompositionSettings->MaxHullsPerGeometry);
					}

					int32 NumHulls = Decomposition.NumHulls();
					if ((NumHulls > 0 && !Hull) || NumHulls > 1)
					{
						TArray<Chaos::FConvexPtr> DecompHulls; DecompHulls.Reserve(NumHulls);
						TArray<FVector> Pivots; Pivots.Reserve(NumHulls);
						for (int32 HullIdx = 0; HullIdx < NumHulls; ++HullIdx)
						{
							TArray<FVector3f> OrigDecompVerts = Decomposition.GetVertices<float>(HullIdx);
							// convert to the vector type expected by Chaos::FConvex
							TArray<::Chaos::FConvex::FVec3Type> DecompVerts;
							DecompVerts.SetNumUninitialized(OrigDecompVerts.Num());
							for (int32 PtIdx = 0; PtIdx < OrigDecompVerts.Num(); ++PtIdx)
							{
								DecompVerts[PtIdx] = OrigDecompVerts[PtIdx];
							}
							FilterHullPoints(DecompVerts, SimplificationDistanceThreshold);
							FVector Pivot = ScaleHullPoints(DecompVerts, OverlapRemovalShrinkPercent);
							Pivots.Add(Pivot);
							DecompHulls.Add(Chaos::FConvexPtr(new Chaos::FConvex(DecompVerts, UE_KINDA_SMALL_NUMBER)));
						}
						HullCS.Lock();
						for (int32 HullIdx = 0; HullIdx < DecompHulls.Num(); ++HullIdx)
						{
							int32 ConvexIdx = Convexes.Add(MoveTemp(DecompHulls[HullIdx]));
							AddPivot(Convexes, ConvexPivots, Pivots[HullIdx]);
							TransformToConvexIndices[Idx].Add(ConvexIdx);
						}
						HullCS.Unlock();
						return;
					}
				}
			}
			// We don't need a convex decomposition
			if (!Hull)
			{
				Hull = ComputeHull(HullPivot);
			}
			HullCS.Lock();
			int32 ConvexIdx = Convexes.Add(MoveTemp(Hull));
			AddPivot(Convexes, ConvexPivots, HullPivot);
			HullCS.Unlock();
			TransformToConvexIndices[Idx].Add(ConvexIdx);
		}
	});
}

void TransformHullsToLocal(
	TArray<FTransform>& GlobalTransformArray,
	TArray<Chaos::FConvexPtr>& Convexes,
	TArray<FVector>& ConvexPivots,
	TArray<TSet<int32>>& TransformToConvexIndices,
	double OverlapRemovalShrinkPercent
)
{
	checkSlow((OverlapRemovalShrinkPercent == 0.0  && ConvexPivots.Num() == 0) || Convexes.Num() == ConvexPivots.Num());
	
	double ScaleFactor = 1.0 - OverlapRemovalShrinkPercent / 100.0;
	double InvScaleFactor = 1.0;
	if (ensure(ScaleFactor != 0.0))
	{
		InvScaleFactor = 1.0 / ScaleFactor;
	}

	ParallelFor(TransformToConvexIndices.Num(), [&](int32 Bone)
	{
		FTransform& Transform = GlobalTransformArray[Bone];
		TArray<Chaos::FConvex::FVec3Type> HullPts;
		for (int32 ConvexIdx : TransformToConvexIndices[Bone])
		{
			HullPts.Reset();
			for (const Chaos::FConvex::FVec3Type& P : Convexes[ConvexIdx]->GetVertices())
			{
				FVector PVec(P);
				if (OverlapRemovalShrinkPercent != 0.0)
				{

					PVec = (PVec - ConvexPivots[ConvexIdx]) * InvScaleFactor + ConvexPivots[ConvexIdx];
				}
				HullPts.Add(FVector(Transform.InverseTransformPosition(PVec)));
			}
			// Do not simplify hulls when we're just trying to transform them
			*Convexes[ConvexIdx] = Chaos::FConvex(HullPts, UE_KINDA_SMALL_NUMBER);
		}
	});
}

// copy hulls from InBone to OutBone
// Store the results in a temporary array, to be added in bulk later
bool CopyHulls(
	const TArray<FTransform>& InGlobalTransformArray,
	const TManagedArray<Chaos::FConvexPtr>& InConvexes,
	const TManagedArray<TSet<int32>>& InTransformToConvexIndices,
	int32 InBone,
	const TArray<FTransform>& OutGlobalTransformArray,
	TArray<Chaos::FConvexPtr>& OutConvexes,
	TArray<TSet<int32>>& OutTransformToConvexIndices,
	int32 OutBone
)
{
	const FTransform& InTransform = InGlobalTransformArray[InBone];
	const FTransform& OutTransform = InGlobalTransformArray[OutBone];
	TArray<Chaos::FConvex::FVec3Type> HullPts;
	for (int32 ConvexIdx : InTransformToConvexIndices[InBone])
	{
		HullPts.Reset();
		for (const Chaos::FConvex::FVec3Type& P : InConvexes[ConvexIdx]->GetVertices())
		{
			HullPts.Add(OutTransform.InverseTransformPosition(InTransform.TransformPosition(FVector(P))));
		}
		// Do not simplify hulls when we're just trying to transform them
		int32 OutIdx = OutConvexes.Add(Chaos::FConvexPtr(new Chaos::FConvex(HullPts, UE_KINDA_SMALL_NUMBER)));
		OutTransformToConvexIndices[OutBone].Add(OutIdx);
	}

	return true;
}

// enum to pass to AddHullProximity's GetTransformBFn to indicate that all hulls use the same constant FTransform, so we can skip transforms and keep hulls in the shared space
enum class EUseConstantTransform
{
	ConstantTransform
};

// Helper to add hull proximities for one hull to a set of neighboring hulls
template<typename GetOtherTransformFn, typename HullIndexMapContainer = TArray<int32>>
static void AddHullProximity(TArray<TPair<int32, int32>>& HullProximity, const EConvexHullProximityFilter ProximityFilter, const float ProximityDistanceThreshold,
	TFunctionRef<const Chaos::FConvex* (int32)> GetConvex, const FTransform& TransformA, int32 ConvexA,
	GetOtherTransformFn GetTransformBFn, int32 ConvexBIdxStart, int32 ConvexBIdxEnd, const HullIndexMapContainer* ConvexBIdxMap = nullptr)
{
	Chaos::FConvex::FAABB3Type BoxA;
	if (ProximityFilter == EConvexHullProximityFilter::BoundingBox)
	{
		BoxA = GetConvex(ConvexA)->GetLocalBoundingBox();
		if constexpr (!std::is_same_v<GetOtherTransformFn, EUseConstantTransform>)
		{
			BoxA = BoxA.TransformedAABB(TransformA);
			BoxA.Thicken(ProximityDistanceThreshold);
		}
		else
		{
			float ThickenScaleFactor = 1.0f / FMath::Max(FMathf::ZeroTolerance, (float)TransformA.GetScale3D().GetAbsMax());
			BoxA.Thicken(ProximityDistanceThreshold * ThickenScaleFactor);
		}
	}
	for (int32 ConvexBIdx = ConvexBIdxStart; ConvexBIdx < ConvexBIdxEnd; ++ConvexBIdx)
	{
		const int32 ConvexB = ConvexBIdxMap ? (*ConvexBIdxMap)[ConvexBIdx] : ConvexBIdx;
		if (ProximityFilter == EConvexHullProximityFilter::BoundingBox)
		{
			Chaos::FConvex::FAABB3Type BoxB = GetConvex(ConvexB)->GetLocalBoundingBox();
			if constexpr (!std::is_same_v<GetOtherTransformFn, EUseConstantTransform>)
			{
				BoxB = BoxB.TransformedAABB(GetTransformBFn(ConvexB));
			}
			if (!BoxA.Intersects(BoxB))
			{
				continue;
			}
		}
		HullProximity.Emplace(ConvexA, ConvexB);
	}
}

}

void FGeometryCollectionConvexUtility::CreateConvexHullAttributesIfNeeded(FManagedArrayCollection& Collection)
{
	if (!Collection.HasGroup(FGeometryCollection::ConvexGroup))
	{
		Collection.AddGroup(FGeometryCollection::ConvexGroup);
	}

	if (!Collection.HasAttribute("TransformToConvexIndices", FTransformCollection::TransformGroup))
	{
		FManagedArrayCollection::FConstructionParameters ConvexDependency(FGeometryCollection::ConvexGroup);
		Collection.AddAttribute<TSet<int32>>("TransformToConvexIndices", FTransformCollection::TransformGroup, ConvexDependency);
	}

	if (!Collection.HasAttribute(FGeometryCollection::ConvexHullAttribute, FGeometryCollection::ConvexGroup))
	{
		Collection.AddAttribute<Chaos::FConvexPtr>(FGeometryCollection::ConvexHullAttribute, FGeometryCollection::ConvexGroup);
	}
}

UE::GeometryCollectionConvexUtility::FConvexHulls
FGeometryCollectionConvexUtility::ComputeLeafHulls(FGeometryCollection* GeometryCollection, const TArray<FTransform>& GlobalTransformArray, double SimplificationDistanceThreshold, double OverlapRemovalShrinkPercent,
	TFunction<bool(int32)> SkipBoneFn, const FConvexDecompositionSettings* OptionalDecompositionSettings,
	const TArray<FTransformedConvex>* OptionalIntersectConvexHulls,
	const TArray<TSet<int32>>* OptionalTransformToIntersectHulls,
	TArray<FGeometryCollectionConvexUtility::FSphereCoveringInfo>* OutComputedNavigableSpheres)
{
	check(GeometryCollection);


	UE::GeometryCollectionConvexUtility::FConvexHulls Hulls;

	// Prevent ~100% shrink percentage, because it would not be reversible ...
	// Note: Could alternatively just disable convex overlap removal in this case
	Hulls.OverlapRemovalShrinkPercent = FMath::Min(99.9, OverlapRemovalShrinkPercent);

	TManagedArray<int32>* CustomConvexFlags = GetCustomConvexFlags(GeometryCollection, false);
	auto ConvexFlagsAlwaysFalse = [](int32) -> bool { return false; };
	auto ConvexFlagsFromArray = [&CustomConvexFlags](int32 TransformIdx) -> bool { return (bool)(*CustomConvexFlags)[TransformIdx]; };
	TFunctionRef<bool(int32)> HasCustomConvexFn = (CustomConvexFlags != nullptr) ? (TFunctionRef<bool(int32)>)ConvexFlagsFromArray : (TFunctionRef<bool(int32)>)ConvexFlagsAlwaysFalse;

	HullsFromGeometry(*GeometryCollection, GlobalTransformArray, HasCustomConvexFn, Hulls.Hulls, Hulls.Pivots, Hulls.TransformToHullsIndices, GeometryCollection->SimulationType,
		FGeometryCollection::ESimulationTypes::FST_Rigid, SimplificationDistanceThreshold, Hulls.OverlapRemovalShrinkPercent, SkipBoneFn, OptionalDecompositionSettings,
		OptionalIntersectConvexHulls, OptionalTransformToIntersectHulls, OutComputedNavigableSpheres
	);

	return Hulls;
}

FGeometryCollectionConvexUtility::FGeometryCollectionConvexData FGeometryCollectionConvexUtility::CreateNonOverlappingConvexHullData(
	FGeometryCollection* GeometryCollection, double FracAllowRemove, double SimplificationDistanceThreshold, double CanExceedFraction, EConvexOverlapRemoval OverlapRemovalMethod,
	double OverlapRemovalShrinkPercentIn, UE::GeometryCollectionConvexUtility::FConvexHulls* PrecomputedLeafHulls)
{
	check(GeometryCollection);

	TManagedArray<int32>* CustomConvexFlags = GetCustomConvexFlags(GeometryCollection, false);
	auto ConvexFlagsAlwaysFalse = [](int32) -> bool { return false; };
	auto ConvexFlagsFromArray = [&CustomConvexFlags](int32 TransformIdx) -> bool { return (bool)(*CustomConvexFlags)[TransformIdx]; };
	TFunctionRef<bool(int32)> HasCustomConvexFn = (CustomConvexFlags != nullptr) ? (TFunctionRef<bool(int32)>)ConvexFlagsFromArray : (TFunctionRef<bool(int32)>)ConvexFlagsAlwaysFalse;

	TArray<FTransform> GlobalTransformArray;
	GeometryCollectionAlgo::GlobalMatrices(GeometryCollection->Transform, GeometryCollection->Parent, GlobalTransformArray);

	UE::GeometryCollectionConvexUtility::FConvexHulls* UseLeafHulls = PrecomputedLeafHulls;
	UE::GeometryCollectionConvexUtility::FConvexHulls LocalLeafHullsStorage;
	if (!UseLeafHulls)
	{
		LocalLeafHullsStorage = ComputeLeafHulls(GeometryCollection, GlobalTransformArray, SimplificationDistanceThreshold, OverlapRemovalShrinkPercentIn);
		UseLeafHulls = &LocalLeafHullsStorage;
	}

	FGeometryCollectionProximityUtility ProximityUtility(GeometryCollection);
	ProximityUtility.RequireProximity(UseLeafHulls);
	SetVolumeAttributes(GeometryCollection);

	const TManagedArray<TSet<int32>>* GCProximity = GeometryCollection->FindAttribute<TSet<int32>>("Proximity", FGeometryCollection::GeometryGroup);
	const TManagedArray<float>* Volume = GeometryCollection->FindAttribute<float>("Volume", FGeometryCollection::TransformGroup);

	CreateNonoverlappingConvexHulls(UseLeafHulls->Hulls, UseLeafHulls->Pivots, UseLeafHulls->TransformToHullsIndices, HasCustomConvexFn, GeometryCollection->SimulationType, FGeometryCollection::ESimulationTypes::FST_Rigid, FGeometryCollection::ESimulationTypes::FST_None,
		GeometryCollection->Parent, GCProximity, GeometryCollection->TransformIndex, Volume, FracAllowRemove, SimplificationDistanceThreshold, CanExceedFraction, OverlapRemovalMethod, UseLeafHulls->OverlapRemovalShrinkPercent);

	TransformHullsToLocal(GlobalTransformArray, UseLeafHulls->Hulls, UseLeafHulls->Pivots, UseLeafHulls->TransformToHullsIndices, UseLeafHulls->OverlapRemovalShrinkPercent);

	CreateConvexHullAttributesIfNeeded(*GeometryCollection);

	TManagedArray<TSet<int32>>& TransformToConvexIndices = GeometryCollection->ModifyAttribute<TSet<int32>>("TransformToConvexIndices", FTransformCollection::TransformGroup);
	TManagedArray<Chaos::FConvexPtr>& ConvexHull = GeometryCollection->ModifyAttribute<Chaos::FConvexPtr>(FGeometryCollection::ConvexHullAttribute, FGeometryCollection::ConvexGroup);
	TransformToConvexIndices = MoveTemp(UseLeafHulls->TransformToHullsIndices);
	GeometryCollection->EmptyGroup(FGeometryCollection::ConvexGroup);
	GeometryCollection->Resize(UseLeafHulls->Hulls.Num(), FGeometryCollection::ConvexGroup);
	ConvexHull = MoveTemp(UseLeafHulls->Hulls);

	// clear all null and empty hulls
	RemoveEmptyConvexHulls(*GeometryCollection);

	checkSlow(FGeometryCollectionConvexUtility::ValidateConvexData(GeometryCollection));

	return { TransformToConvexIndices, ConvexHull };
}

void FGeometryCollectionConvexUtility::GenerateLeafConvexHulls(
	FGeometryCollection& Collection, bool bRestrictToSelection, 
	const TArrayView<const int32> TransformSubset, const FLeafConvexHullSettings& Settings,
	TArray<FSphereCoveringInfo>* OutComputedNavigableSpheres

)
{
	int32 NumTransforms = Collection.NumElements(FGeometryCollection::TransformGroup);

	const TManagedArrayAccessor<Chaos::FImplicitObjectPtr> ExternalCollisionAttribute(Collection, FGeometryCollection::ExternalCollisionsAttribute, FGeometryCollection::TransformGroup);
	bool bHasExternalCollision = ExternalCollisionAttribute.IsValid();
	if (bHasExternalCollision)
	{
		bool bAnyValid = false;
		for (int32 TransformIdx = 0; TransformIdx < NumTransforms; ++TransformIdx)
		{
			bAnyValid = bAnyValid || ExternalCollisionAttribute[TransformIdx].IsValid();
		}
		if (!bAnyValid)
		{
			bHasExternalCollision = false;
		}
	}
	bool bUseExternal = bHasExternalCollision && (Settings.GenerateMethod == EGenerateConvexMethod::ExternalCollision || Settings.GenerateMethod == EGenerateConvexMethod::IntersectExternalWithComputed);
	bool bUseGenerated = !bUseExternal || Settings.GenerateMethod == EGenerateConvexMethod::IntersectExternalWithComputed;
	bool bUseIntersect = bUseExternal && Settings.GenerateMethod == EGenerateConvexMethod::IntersectExternalWithComputed;
	
	UE::GeometryCollectionConvexUtility::FConvexHulls ComputedHulls;
	TArray<FTransformedConvex> ExternalHulls;
	TArray<TSet<int32>> TransformToExternalHullsIndices;
	TransformToExternalHullsIndices.SetNum(NumTransforms);

	CreateConvexHullAttributesIfNeeded(Collection);

	auto ComputeTransformedHull = [](const Chaos::FConvex& HullIn, const FTransform& TransformIn) -> Chaos::FConvexPtr
	{
		FTransform3f Transform = (FTransform3f)TransformIn;
		TArray<Chaos::FConvex::FVec3Type> HullPts;
		for (const Chaos::FConvex::FVec3Type& P : HullIn.GetVertices())
		{
			HullPts.Add(Transform.TransformPosition(P));
		}
		return Chaos::FConvexPtr(new Chaos::FConvex(HullPts, UE_KINDA_SMALL_NUMBER));
	};

	TArray<int32> LocalTransformIndices;
	if (!bRestrictToSelection)
	{
		LocalTransformIndices.Reserve(NumTransforms);
		for (int32 Idx = 0; Idx < NumTransforms; ++Idx)
		{
			if (Collection.SimulationType[Idx] == FGeometryCollection::ESimulationTypes::FST_Rigid)
			{
				LocalTransformIndices.Add(Idx);
			}
		}
	}
	const TArrayView<const int32> UseTransforms = bRestrictToSelection ? TransformSubset : TArrayView<const int32>(LocalTransformIndices);
	if (UseTransforms.IsEmpty())
	{
		return;
	}

	TManagedArray<Chaos::FConvexPtr>& ConvexHullAttrib = Collection.ModifyAttribute<Chaos::FConvexPtr>(FGeometryCollection::ConvexHullAttribute, FGeometryCollection::ConvexGroup);
	TManagedArray<TSet<int32>>& TransformToConvexIndicesAttrib = Collection.ModifyAttribute<TSet<int32>>("TransformToConvexIndices", FTransformCollection::TransformGroup);
	
	// Remove all convex hulls from bones that we have selected for re-compute
	for (int32 TransformIdx : UseTransforms)
	{
		if (Collection.SimulationType[TransformIdx] != FGeometryCollection::ESimulationTypes::FST_Rigid)
		{
			continue;
		}
		TransformToConvexIndicesAttrib[TransformIdx].Empty();
	}
	// Remove all convex hulls that are no longer referenced by any transforms
	{
		TArray<bool> ConvexHullsUsed; ConvexHullsUsed.SetNumZeroed(ConvexHullAttrib.Num());
		for (int32 TransformIdx = 0; TransformIdx < TransformToConvexIndicesAttrib.Num(); ++TransformIdx)
		{
			for (int32 ConvexIdx : TransformToConvexIndicesAttrib[TransformIdx])
			{
				ConvexHullsUsed[ConvexIdx] = true;
			}
		}
		for (int32 ConvexIdx = 0; ConvexIdx < ConvexHullAttrib.Num(); ++ConvexIdx)
		{
			if (!ConvexHullsUsed[ConvexIdx])
			{
				ConvexHullAttrib[ConvexIdx].SafeRelease();
			}
		}
		RemoveEmptyConvexHulls(Collection);
	}
	
	auto AddComputedHullsToCollection = [&Collection, &ConvexHullAttrib, &TransformToConvexIndicesAttrib](TArray<Chaos::FConvexPtr>& Hulls, TArray<TSet<int32>>& TransformToHullsIndices)
	{
		int32 InitialNum = ConvexHullAttrib.Num();
		Collection.Resize(InitialNum + Hulls.Num(), FGeometryCollection::ConvexGroup);
		for (int32 Idx = 0; Idx < Hulls.Num(); ++Idx)
		{
			ConvexHullAttrib[InitialNum + Idx] = MoveTemp(Hulls[Idx]);
		}
		for (int32 TransformIdx = 0; TransformIdx < TransformToHullsIndices.Num(); ++TransformIdx)
		{
			for (int32 ConvexIdx : TransformToHullsIndices[TransformIdx])
			{
				TransformToConvexIndicesAttrib[TransformIdx].Add(ConvexIdx + InitialNum);
			}
		}
	};

	if (bUseExternal)
	{
		for (int32 SourceTransformIdx : UseTransforms)
		{
			if (Collection.SimulationType[SourceTransformIdx] != FGeometryCollection::ESimulationTypes::FST_Rigid)
			{
				continue;
			}

			// convert the external collisions to convex hulls
			const Chaos::FImplicitObjectPtr ExternalCollision = ExternalCollisionAttribute[SourceTransformIdx];
			if (!ExternalCollision.IsValid())
			{
				continue;
			}

			const int32 ExternalHullsStart = ExternalHulls.Num();
			ConvertImplicitToConvexArray(*ExternalCollision, FTransform::Identity, ExternalHulls);
			for (int32 HullIndex = ExternalHullsStart; HullIndex < ExternalHulls.Num(); HullIndex++)
			{
				TransformToExternalHullsIndices[SourceTransformIdx].Add(HullIndex);
			}
		}

		if (!bUseIntersect)
		{
			TArray<Chaos::FConvexPtr> FinalHulls;
			FinalHulls.SetNum(ExternalHulls.Num());

			// Transform the hulls to the local space of the transform
			ParallelFor(ExternalHulls.Num(), [&](int32 HullIdx)
			{
				FinalHulls[HullIdx] = ComputeTransformedHull(*ExternalHulls[HullIdx].Convex, ExternalHulls[HullIdx].Transform);
			});

			AddComputedHullsToCollection(FinalHulls, TransformToExternalHullsIndices);
		}
	}
	
	if (bUseGenerated)
	{
		// Identity Transform Array allows us to leave leaf hulls in their original local coordinate spaces
		TArray<FTransform> IdentityTransformArray;
		IdentityTransformArray.Init(FTransform::Identity, NumTransforms);
		TFunction<bool(int32)> SkipBoneFn = nullptr;
		TSet<int32> SelectionSet;
		if (bRestrictToSelection)
		{
			SelectionSet.Append(UseTransforms);
			SkipBoneFn = [SelectionSet](int32 BoneIdx) -> bool
			{
				return SelectionSet.Contains(BoneIdx);
			};
		}
		ComputedHulls = ComputeLeafHulls(&Collection, IdentityTransformArray, Settings.SimplificationDistanceThreshold, 0, SkipBoneFn, &Settings.DecompositionSettings,
			Settings.bComputeIntersectionsBeforeHull ? &ExternalHulls : nullptr, 
			Settings.bComputeIntersectionsBeforeHull ? &TransformToExternalHullsIndices : nullptr,
			OutComputedNavigableSpheres
		);

		if (!bUseIntersect)
		{
			AddComputedHullsToCollection(ComputedHulls.Hulls, ComputedHulls.TransformToHullsIndices);
		}
	}
	
	if (bUseIntersect)
	{
		TArray<Chaos::FConvexPtr> FinalHulls;
		int32 HullIndexOffset = ConvexHullAttrib.Num();
		for (int32 SourceTransformIdx : UseTransforms)
		{
			if (Collection.SimulationType[SourceTransformIdx] != FGeometryCollection::ESimulationTypes::FST_Rigid)
			{
				continue;
			}

			TSet<int32>& GeoHulls = ComputedHulls.TransformToHullsIndices[SourceTransformIdx];

			if (Settings.IntersectFilters.OnlyIntersectIfComputedIsSmallerFactor < 1.0 || Settings.IntersectFilters.MinExternalVolumeToIntersect > 0.0)
			{
				double ComputedVolSum = 0.0;
				for (int32 GeoHullIdx : GeoHulls)
				{
					ComputedVolSum += (double)ComputedHulls.Hulls[GeoHullIdx]->GetVolume();
				}
				double ExternalVolSum = 0.0;
				for (int32 ExtHull : TransformToExternalHullsIndices[SourceTransformIdx])
				{
					ExternalVolSum += (double)ExternalHulls[ExtHull].Convex->GetVolume();
				}
				if (
					(Settings.IntersectFilters.OnlyIntersectIfComputedIsSmallerFactor < 1.0 && ComputedVolSum >= ExternalVolSum * Settings.IntersectFilters.OnlyIntersectIfComputedIsSmallerFactor) ||
					(ExternalVolSum < Settings.IntersectFilters.MinExternalVolumeToIntersect)
					)
				{
					// Filters allow us to skip computing the intersection and just use the external hulls
					for (int32 ExtHull : TransformToExternalHullsIndices[SourceTransformIdx])
					{
						int32 HullIdx = FinalHulls.Add(ComputeTransformedHull(*ExternalHulls[ExtHull].Convex, ExternalHulls[ExtHull].Transform));
						TransformToConvexIndicesAttrib[SourceTransformIdx].Add(HullIndexOffset + HullIdx);
					}
					continue;
				}
			}

			// We've already computed the hulls w/ external geo intersection; just move it to the output
			if (Settings.bComputeIntersectionsBeforeHull)
			{
				for (int32 GeoHullIdx : GeoHulls)
				{
					int32 HullIdx = FinalHulls.Add(MoveTemp(ComputedHulls.Hulls[GeoHullIdx]));
					TransformToConvexIndicesAttrib[SourceTransformIdx].Add(HullIdx + HullIndexOffset);
				}
				continue;
			}

			// The !bComputeIntersectionsBeforeHull path
			checkSlow(!Settings.bComputeIntersectionsBeforeHull);
			for (int32 GeoHullIdx : GeoHulls)
			{
				if (TransformToExternalHullsIndices.IsEmpty())
				{
					TransformToConvexIndicesAttrib[SourceTransformIdx].Add(HullIndexOffset + FinalHulls.Add(MoveTemp(ComputedHulls.Hulls[GeoHullIdx])));

					continue;
				}
				for (int32 ExtHull : TransformToExternalHullsIndices[SourceTransformIdx])
				{
					int32 HullIdx = FinalHulls.Add(Chaos::FConvexPtr(new Chaos::FConvex()));
					UE::GeometryCollectionConvexUtility::IntersectConvexHulls(FinalHulls[HullIdx].GetReference(),
						ComputedHulls.Hulls[GeoHullIdx].GetReference(), 0.0f, ExternalHulls[ExtHull].Convex.GetReference(),
						nullptr, &ExternalHulls[ExtHull].Transform, &ExternalHulls[ExtHull].Transform, Settings.SimplificationDistanceThreshold);
					TransformToConvexIndicesAttrib[SourceTransformIdx].Add(HullIdx + HullIndexOffset);
				}
			}
		}
		// copy intersected hulls into the output hull attrib
		Collection.Resize(HullIndexOffset + FinalHulls.Num(), FGeometryCollection::ConvexGroup);
		for (int32 Idx = 0; Idx < FinalHulls.Num(); ++Idx)
		{
			ConvexHullAttrib[HullIndexOffset + Idx] = MoveTemp(FinalHulls[Idx]);
		}
	}

	RemoveEmptyConvexHulls(Collection);
}

void FGeometryCollectionConvexUtility::GenerateClusterConvexHullsFromChildrenHulls(FGeometryCollection& Collection, const FClusterConvexHullSettings& Settings, const TArrayView<const int32> TransformSubset)
{
	GenerateClusterConvexHullsFromLeafOrChildrenHullsInternal(Collection, Settings, true, true/*bUseDirectChildren*/, TransformSubset);
}
void FGeometryCollectionConvexUtility::GenerateClusterConvexHullsFromChildrenHulls(FGeometryCollection& Collection, const FClusterConvexHullSettings& Settings)
{
	GenerateClusterConvexHullsFromLeafOrChildrenHullsInternal(Collection, Settings, false, true/*bUseDirectChildren*/, TArrayView<const int32>());
}
void FGeometryCollectionConvexUtility::GenerateClusterConvexHullsFromLeafHulls(FGeometryCollection& Collection, const FClusterConvexHullSettings& Settings, const TArrayView<const int32> TransformSubset)
{
	GenerateClusterConvexHullsFromLeafOrChildrenHullsInternal(Collection, Settings, true, false/*bUseDirectChildren*/, TransformSubset);
}
void FGeometryCollectionConvexUtility::GenerateClusterConvexHullsFromLeafHulls(FGeometryCollection& Collection, const FClusterConvexHullSettings& Settings)
{
	GenerateClusterConvexHullsFromLeafOrChildrenHullsInternal(Collection, Settings, false, false/*bUseDirectChildren*/, TArrayView<const int32>());
}

void FGeometryCollectionConvexUtility::GenerateClusterConvexHullsFromLeafOrChildrenHullsInternal(FGeometryCollection& Collection, const FClusterConvexHullSettings& Settings, bool bOnlySubset, bool bUseDirectChildren, const TArrayView<const int32> TransformSubset)
{
	static FName ConvexGroupName = FGeometryCollection::ConvexGroup;
	static FName ConvexHullAttributeName = FGeometryCollection::ConvexHullAttribute;
	static FName TransformToConvexIndicesName("TransformToConvexIndices");
	
	CreateConvexHullAttributesIfNeeded(Collection);

	TManagedArrayAccessor<Chaos::FConvexPtr> ConvexHullAttribute(Collection, ConvexHullAttributeName, ConvexGroupName);
	TManagedArrayAccessor<TSet<int32>> TransformToConvexIndicesAttribute(Collection, TransformToConvexIndicesName, FGeometryCollection::TransformGroup);
	const TManagedArrayAccessor<Chaos::FImplicitObjectPtr> ExternalCollisionAttribute(Collection, FGeometryCollection::ExternalCollisionsAttribute, FGeometryCollection::TransformGroup);

	GeometryCollection::Facades::FCollectionTransformFacade TransformFacade(Collection);
	Chaos::Facades::FCollectionHierarchyFacade HierarchyFacade(Collection);

	FGeometryCollectionProximityUtility ProximityUtility(&Collection);
	ProximityUtility.RequireProximity();
	const TManagedArray<TSet<int32>>& Proximity = Collection.GetAttribute<TSet<int32>>("Proximity", FGeometryCollection::GeometryGroup);

	// Whether to consider directly merging hulls from the same child/leaf transform; without this, these hulls can still be merged if they are also merged to a neighboring hull
	// TODO: decide whether to always-enable this, or expose this option to the user, or leave it disabled
	constexpr bool bConsiderIntraTransformConvexMerges = false;

	const TArray<FTransform> GlobalTransforms = TransformFacade.ComputeCollectionSpaceTransforms();
	const TManagedArrayAccessor<int32> SimulationTypeAttribute(Collection, FGeometryCollection::SimulationTypeAttribute, FTransformCollection::TransformGroup);
	const TArray<int32> DepthFirstTransformIndices = HierarchyFacade.GetTransformArrayInDepthFirstOrder();
	TArray<int32> TransformsToProcess;
	if (bOnlySubset)
	{
		if (TransformSubset.IsEmpty())
		{
			return;
		}

		if (bUseDirectChildren || Settings.bAllowMergingLeafHulls)
		{
			// If using direct children, or merging leaf hulls, order matters -- use depth-first order
			TSet<int32> ToProcessSet;
			ToProcessSet.Append(TransformSubset);
			TransformsToProcess.Reserve(TransformSubset.Num());
			for (int32 TransformIdx : DepthFirstTransformIndices)
			{
				if (ToProcessSet.Contains(TransformIdx))
				{
					TransformsToProcess.Add(TransformIdx);
				}
			}
		}
		else
		{
			// Order doesn't matter
			TransformsToProcess.Append(TransformSubset);
		}
	}
	else // process all
	{
		TransformsToProcess = DepthFirstTransformIndices;
	}

	// Simulation types must be present
	if (!SimulationTypeAttribute.IsValid())
	{
		return;
	}

	for (int32 TransformIndex : TransformsToProcess)
	{
		if (TransformIndex < 0 || TransformIndex >= SimulationTypeAttribute.Num())
		{
			UE_LOGF(LogChaos, Warning, "Transform Index %d out of bounds [0,%d)", TransformIndex, SimulationTypeAttribute.Num());
			continue;
		}
		// only do this for clusters, unless we allow merging for leaf hulls too
		const bool bIsCluster = (SimulationTypeAttribute.Get()[TransformIndex] == FGeometryCollection::ESimulationTypes::FST_Clustered);
		if (bIsCluster || Settings.bAllowMergingLeafHulls)
		{
			TArray<int32> SourceTransformIndices;
			if (!bIsCluster)
			{
				SourceTransformIndices.Add(TransformIndex);
			}
			else if (bUseDirectChildren)
			{
				SourceTransformIndices = HierarchyFacade.GetChildrenAsArray(TransformIndex);
				// TODO: consider expanding this recursively to children for any nodes without convex hulls, until we hit a leaf/rigid node
			}
			else
			{
				FGeometryCollectionClusteringUtility::GetLeafBones(&Collection, TransformIndex, true, SourceTransformIndices);
			}
			
			if (SourceTransformIndices.Num() > 0)
			{
				const FTransform& ParentTransform = GlobalTransforms[TransformIndex];

				TManagedArray<TSet<int32>>& TransformToConvexIndices = TransformToConvexIndicesAttribute.Modify();
				TManagedArray<Chaos::FConvexPtr>& ConvexHull = ConvexHullAttribute.Modify();

				struct FHullInfo
				{
					Chaos::FConvex* Convex = nullptr;
					FTransform Transform;
				};

				// build information required for merging hulls 
				TArray<FHullInfo> Hulls;
				TArray<FTransformedConvex> ExternalHulls;
				TMap<int32, TArray<int32>> TransformToHullIdx;
				TArray<TPair<int32, int32>> HullProximity;
	
				for (int32 SourceTransformIndex : SourceTransformIndices)
				{
					const FTransform InnerTransform = GlobalTransforms[SourceTransformIndex];
					FTransform ChildToParentTransform = InnerTransform.GetRelativeTransform(ParentTransform);

					const Chaos::FImplicitObjectPtr ExternalCollision = (ExternalCollisionAttribute.IsValid()) ? ExternalCollisionAttribute.Get()[SourceTransformIndex] : Chaos::FImplicitObjectPtr();
					if (ExternalCollision && Settings.bUseExternalCollisionIfAvailable)
					{
						const int32 ExternalHullsStart = ExternalHulls.Num();
						ConvertImplicitToConvexArray(*ExternalCollision, FTransform::Identity, ExternalHulls);
						for (int32 HullIndex = ExternalHullsStart; HullIndex < ExternalHulls.Num();  HullIndex++)
						{
							const FTransformedConvex& ExternalHull = ExternalHulls[HullIndex];

							FHullInfo HullInfo;
							HullInfo.Convex = ExternalHull.Convex.GetReference();
							HullInfo.Transform = ExternalHull.Transform * ChildToParentTransform;
							const int32 HullIdx = Hulls.Emplace(HullInfo);
							TransformToHullIdx.FindOrAdd(SourceTransformIndex).Add(HullIdx);
						}
					}
					else
					{
						for (int32 SourceConvexIdx : TransformToConvexIndices[SourceTransformIndex])
						{
							FHullInfo HullInfo;
							HullInfo.Convex = ConvexHull[SourceConvexIdx].GetReference();
							HullInfo.Transform = ChildToParentTransform;
							const int32 HullIdx = Hulls.Emplace(HullInfo);
							TransformToHullIdx.FindOrAdd(SourceTransformIndex).Add(HullIdx);
						}
					}
				}
				// Set HullProximity for convex merges -- the connections between convex hulls that can be merged
				if (!bIsCluster || bUseDirectChildren || Settings.AllowMergesMethod == EAllowConvexMergeMethod::Any)
				{
					// With MergeMethod of Any, any pair of convex hulls can be merged
					// Note: Currently bUseDirectChildren only supports this method
					// This is likely to be slower and may consider additional merges that the proximity method cannot
					for (int32 IndexA = 0; IndexA < SourceTransformIndices.Num(); IndexA++)
					{
						const int32 TransformIndexA = SourceTransformIndices[IndexA];
						if (const TArray<int32>* HullIndicesA = TransformToHullIdx.Find(TransformIndexA))
						{
							// Consider merges between convex hulls from separate transforms
							for (int32 IndexB = IndexA + 1; IndexB < SourceTransformIndices.Num(); IndexB++)
							{
								const int32 TransformIndexB = SourceTransformIndices[IndexB];
								if (const TArray<int32>* HullIndicesB = TransformToHullIdx.Find(TransformIndexB))
								{
									for (int32 SourceHullIdxA : (*HullIndicesA))
									{

										AddHullProximity(HullProximity, Settings.ProximityFilter, Settings.ProximityDistanceThreshold,
											[&Hulls](int32 Idx) -> const Chaos::FConvex* { return Hulls[Idx].Convex; }, Hulls[SourceHullIdxA].Transform,
											SourceHullIdxA, [&Hulls](int32 Idx) -> const FTransform& { return Hulls[Idx].Transform; },
											0, HullIndicesB->Num(), HullIndicesB);
									}
								}
							}
							// Consider merges between convex hulls within a transform
							if (!bIsCluster || bConsiderIntraTransformConvexMerges)
							{
								for (int32 HullSubIdx = 0; HullSubIdx + 1 < HullIndicesA->Num(); ++HullSubIdx)
								{
									const int32 HullA = (*HullIndicesA)[HullSubIdx];
									AddHullProximity(HullProximity, Settings.ProximityFilter, Settings.ProximityDistanceThreshold,
										[&Hulls](int32 Idx) -> const Chaos::FConvex* { return Hulls[Idx].Convex; }, Hulls[HullA].Transform,
										HullA, EUseConstantTransform::ConstantTransform, HullSubIdx + 1, HullIndicesA->Num(), HullIndicesA);
								}
							}
						}
					}
				}
				else // Settings.MergeMethod == EAllowConvexMergeMethod::ByProximity
				{
					for (int32 SourceTransformIndex : SourceTransformIndices)
					{
						const TArray<int32, TInlineAllocator<8>> SourceHullIndices{ TransformToHullIdx.FindOrAdd(SourceTransformIndex) };
						const int32 GeoIdx = Collection.TransformToGeometryIndex[SourceTransformIndex];
						if (GeoIdx > INDEX_NONE)
						{
							// Consider merges between convex hulls from separate transforms
							for (int32 NeighborGeoIndex : Proximity[GeoIdx])
							{
								const int32 NeighborTransformIdx = Collection.TransformIndex[NeighborGeoIndex];
								if (const TArray<int32>* NeighborHullIndices = TransformToHullIdx.Find(NeighborTransformIdx))
								{
									for (int32 SourceHullIdx : SourceHullIndices)
									{
										AddHullProximity(HullProximity, Settings.ProximityFilter, Settings.ProximityDistanceThreshold,
											[&Hulls](int32 Idx) -> const Chaos::FConvex* { return Hulls[Idx].Convex; }, Hulls[SourceHullIdx].Transform,
											SourceHullIdx, [&Hulls](int32 Idx) -> const FTransform& { return Hulls[Idx].Transform; },
											0, NeighborHullIndices->Num(), NeighborHullIndices);

									}
								}
							}
							// Consider merges between convex hulls within a transform
							if (bConsiderIntraTransformConvexMerges)
							{
								for (int32 HullSubIdx = 0; HullSubIdx + 1 < SourceHullIndices.Num(); ++HullSubIdx)
								{
									const int32 HullA = SourceHullIndices[HullSubIdx];
									AddHullProximity(HullProximity, Settings.ProximityFilter, Settings.ProximityDistanceThreshold,
										[&Hulls](int32 Idx) -> const Chaos::FConvex* { return Hulls[Idx].Convex; }, Hulls[HullA].Transform,
										HullA, EUseConstantTransform::ConstantTransform, HullSubIdx + 1, SourceHullIndices.Num(), &SourceHullIndices);
								}
							}
						}
					}
				}

				// try to merge the leaf convex into a simpler set of convex
				const auto GetHullVolume = [&Hulls](int32 Idx) { return (double)Hulls[Idx].Convex->GetVolume(); };
				const auto GetHullNumVertices = [&Hulls](int32 Idx) { return Hulls[Idx].Convex->NumVertices(); };
				const auto GetHullVertex = [&Hulls](int32 Hull, int32 V) {
					const FVector3d LocalVertex(Hulls[Hull].Convex->GetVertex(V));
					return Hulls[Hull].Transform.TransformPosition(LocalVertex);
				};
				UE::Geometry::FConvexDecomposition3 ConvexDecomposition;
				ConvexDecomposition.InitializeFromHulls(Hulls.Num(), GetHullVolume, GetHullNumVertices, GetHullVertex, HullProximity);
				ConvexDecomposition.MergeBest(Settings.ConvexCount, Settings.ErrorToleranceInCm, 0, true /*bAllowCompact*/, false /*bRequireHullTriangles*/, -1 /*MaxHulls*/, Settings.EmptySpace, &ParentTransform);
				
				// reset existing hulls for this transform index
				for (int32 ConvexIndex : TransformToConvexIndices[TransformIndex])
				{
					ConvexHull[ConvexIndex] = nullptr;
				}
				TransformToConvexIndices[TransformIndex].Reset();

				// add new computed ones
				for (int32 DecompIndex = 0; DecompIndex < ConvexDecomposition.Decomposition.Num(); DecompIndex++)
				{
					// Make Implicit Convex of a decomposed element
					TArray<Chaos::FConvex::FVec3Type> Particles;
					const TArray<FVector> Points = ConvexDecomposition.GetVertices<double>(DecompIndex);
					Particles.SetNum(Points.Num());
					for (int32 PointIndex = 0; PointIndex < Points.Num(); PointIndex++)
					{
						Particles[PointIndex] = Points[PointIndex];
					}
					Chaos::FConvexPtr ImplicitConvex( new Chaos::FConvex(MoveTemp(Particles), 0.0f));

					// Add the the element to the union
					const int32 NewConvexIndex = Collection.AddElements(1, ConvexGroupName);
					ConvexHull[NewConvexIndex] = MoveTemp(ImplicitConvex);
					TransformToConvexIndices[TransformIndex].Add(NewConvexIndex);
				}
			}
		}
	}

	// remove empty or null convex hulls
	RemoveEmptyConvexHulls(Collection);

	checkSlow(FGeometryCollectionConvexUtility::ValidateConvexData(&Collection));

}

void FGeometryCollectionConvexUtility::MergeHullsOnTransforms(FManagedArrayCollection& Collection, const FGeometryCollectionConvexUtility::FMergeConvexHullSettings& Settings, bool bRestrictToSelection,
	const TArrayView<const int32> OptionalTransformSelection, UE::Geometry::FSphereCovering* OptionalSphereCoveringOut)
{
	static FName ConvexGroupName = FGeometryCollection::ConvexGroup;

	TOptional<FGeometryCollectionConvexUtility::FGeometryCollectionConvexData> ConvexData = GetConvexHullDataIfPresent(&Collection);
	if (!ConvexData)
	{
		return;
	}

	TManagedArray<TSet<int32>>& TransformToConvexIndices = ConvexData->TransformToConvexIndices;
	TManagedArray<Chaos::FConvexPtr>& ConvexHull = ConvexData->ConvexHull;

	GeometryCollection::Facades::FCollectionTransformFacade TransformFacade(Collection);

	const TArray<FTransform> GlobalTransforms = TransformFacade.ComputeCollectionSpaceTransforms();
	TArray<int32> TransformsToProcess;
	if (bRestrictToSelection)
	{
		TransformsToProcess.Append(OptionalTransformSelection);
	}
	else // process all
	{
		TransformsToProcess.Reserve(TransformToConvexIndices.Num());
		for (int32 Idx = 0; Idx < TransformToConvexIndices.Num(); ++Idx)
		{
			TransformsToProcess.Add(Idx);
		}
	}

	if (OptionalSphereCoveringOut)
	{
		OptionalSphereCoveringOut->Reset();
	}
	if (OptionalSphereCoveringOut && Settings.EmptySpace)
	{
		OptionalSphereCoveringOut->Append(*Settings.EmptySpace);
	}

	for (int32 TransformIndex : TransformsToProcess)
	{
		const int32 InitialNumConvex = TransformToConvexIndices[TransformIndex].Num();
		if (InitialNumConvex <= 1)
		{
			continue;
		}

		TArray<Chaos::FConvex*> Hulls;
		for (int32 ConvexIdx : TransformToConvexIndices[TransformIndex])
		{
			Hulls.Add(ConvexHull[ConvexIdx].GetReference());
		}
		TArray<TPair<int32, int32>> HullProximity;
		for (int32 ConvexA = 0; ConvexA < InitialNumConvex; ++ConvexA)
		{
			AddHullProximity(HullProximity, Settings.ProximityFilter, Settings.ProximityDistanceThreshold,
					[&Hulls](int32 Idx) -> const Chaos::FConvex* { return Hulls[Idx]; }, GlobalTransforms[TransformIndex],
					ConvexA, EUseConstantTransform::ConstantTransform, ConvexA + 1, InitialNumConvex);
		}

		// try to merge the leaf convex into a simpler set of convex
		const auto GetHullVolume = [&Hulls](int32 Idx) { return (double)Hulls[Idx]->GetVolume(); };
		const auto GetHullNumVertices = [&Hulls](int32 Idx) { return Hulls[Idx]->NumVertices(); };
		const auto GetHullVertex = [&Hulls](int32 Hull, int32 V) -> FVector3d {
			const FVector3d LocalVertex(Hulls[Hull]->GetVertex(V));
			return LocalVertex;
		};
		UE::Geometry::FConvexDecomposition3 ConvexDecomposition;
		ConvexDecomposition.InitializeFromHulls(Hulls.Num(), GetHullVolume, GetHullNumVertices, GetHullVertex, HullProximity);
		UE::Geometry::FSphereCovering LocalCovering;
		UE::Geometry::FSphereCovering* UseCovering = Settings.EmptySpace;
		if (Settings.ComputeEmptySpacePerBoneSettings)
		{
			UseCovering = &LocalCovering;
			UE::Geometry::FDynamicMesh3 HullsMesh;
			for (const ::Chaos::FConvex* Hull : Hulls)
			{
				constexpr bool bInvertFaces = false; // don't invert faces, to match orientation expected by AddNegativeSpace below
				AddConvexHullToCompactDynamicMesh(Hull, HullsMesh, &GlobalTransforms[TransformIndex], bInvertFaces);
			}
			UE::Geometry::FDynamicMeshAABBTree3 HullsTree(&HullsMesh, true);
			UE::Geometry::TFastWindingTree<UE::Geometry::FDynamicMesh3> HullsWinding(&HullsTree, true);
			LocalCovering.AddNegativeSpace(HullsWinding, *Settings.ComputeEmptySpacePerBoneSettings, true);

			if (Settings.EmptySpace)
			{
				LocalCovering.Append(*Settings.EmptySpace);
			}

			if (OptionalSphereCoveringOut)
			{
				OptionalSphereCoveringOut->Append(LocalCovering);
			}
		}
		ConvexDecomposition.MergeBest(Settings.MaxConvexCount, Settings.ErrorToleranceInCm, 0, true /*bAllowCompact*/, false /*bRequireHullTriangles*/, Settings.MaxConvexCount /*MaxHulls*/, UseCovering, &GlobalTransforms[TransformIndex]);

		if (!ensure(ConvexDecomposition.Decomposition.Num() > 0))
		{
			// Empty convex decomposition cannot be used / should not happen
			continue;
		}
		if (InitialNumConvex <= ConvexDecomposition.Decomposition.Num())
		{
			// no need to update hulls if the merge didn't change the number of hulls
			continue;
		}

		// reset existing hulls for this transform index
		for (int32 ConvexIndex : TransformToConvexIndices[TransformIndex])
		{
			ConvexHull[ConvexIndex] = nullptr;
		}
		TransformToConvexIndices[TransformIndex].Reset();

		// add new computed ones
		for (int32 DecompIndex = 0; DecompIndex < ConvexDecomposition.Decomposition.Num(); DecompIndex++)
		{
			// Make Implicit Convex of a decomposed element
			TArray<Chaos::FConvex::FVec3Type> Particles;
			const TArray<FVector> Points = ConvexDecomposition.GetVertices<double>(DecompIndex);
			Particles.SetNum(Points.Num());
			for (int32 PointIndex = 0; PointIndex < Points.Num(); PointIndex++)
			{
				Particles[PointIndex] = Points[PointIndex];
			}
			Chaos::FConvexPtr ImplicitConvex( new Chaos::FConvex(MoveTemp(Particles), 0.0f));

			// Add the the element to the union
			const int32 NewConvexIndex = Collection.AddElements(1, ConvexGroupName);
			ConvexHull[NewConvexIndex] = MoveTemp(ImplicitConvex);
			TransformToConvexIndices[TransformIndex].Add(NewConvexIndex);
		}
	}

	// remove empty or null convex hulls
	RemoveEmptyConvexHulls(Collection);

	checkSlow(FGeometryCollectionConvexUtility::ValidateConvexData(&Collection));
}

bool FGeometryCollectionConvexUtility::ValidateConvexData(const FManagedArrayCollection* GeometryCollection)
{
	if (!GeometryCollection->HasAttribute(FGeometryCollection::ConvexHullAttribute, FGeometryCollection::ConvexGroup) || !GeometryCollection->HasAttribute("TransformToConvexIndices", FTransformCollection::TransformGroup))
	{
		return false;
	}
	const TManagedArray<Chaos::FConvexPtr>& ConvexHull = GeometryCollection->GetAttribute<Chaos::FConvexPtr>(FGeometryCollection::ConvexHullAttribute, FGeometryCollection::ConvexGroup);
	for (int32 ConvexIdx = 0; ConvexIdx < ConvexHull.Num(); ConvexIdx++)
	{
		if (!ConvexHull[ConvexIdx].IsValid())
		{
			return false;
		}
	}
	const TManagedArray<TSet<int32>>& TransformToConvexIndices = GeometryCollection->GetAttribute<TSet<int32>>("TransformToConvexIndices", FTransformCollection::TransformGroup);
	for (int32 TransformIdx = 0; TransformIdx < TransformToConvexIndices.Num(); TransformIdx++)
	{
		for (int32 ConvexIdx : TransformToConvexIndices[TransformIdx])
		{
			if (ConvexIdx < 0 || ConvexIdx >= ConvexHull.Num())
			{
				return false;
			}
		}
	}
	return true;
}

Chaos::FConvexPtr FGeometryCollectionConvexUtility::GetConvexHull(const FGeometryCollection* GeometryCollection, int32 GeometryIndex)
{
	check(GeometryCollection);

	int32 VertexCount = GeometryCollection->VertexCount[GeometryIndex];
	int32 VertexStart = GeometryCollection->VertexStart[GeometryIndex];

	TArray<Chaos::FConvex::FVec3Type> Vertices;
	Vertices.SetNum(VertexCount);
	for (int32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
	{
		Vertices[VertexIndex] = GeometryCollection->Vertex[VertexStart+VertexIndex];
	}

	return Chaos::FConvexPtr( new Chaos::FConvex(Vertices, 0.0f));
}


bool FGeometryCollectionConvexUtility::CopyConvexHulls(FManagedArrayCollection& Collection, const TArray<int32>& TransformIndices,
	const FManagedArrayCollection& SourceCollection, const TArray<int32>& SourceTransformIndices, bool bSkipIfEmpty)
{
	static FName ConvexGroupName = FGeometryCollection::ConvexGroup;
	static FName ConvexHullAttributeName = FGeometryCollection::ConvexHullAttribute;
	static FName TransformToConvexIndicesName("TransformToConvexIndices");

	if (TransformIndices.Num() != SourceTransformIndices.Num())
	{
		UE_LOGF(LogChaos, Warning,
			"Attempted to copy convex hulls failed because the source and target transform indices array did not have the same length (%d vs %d)", 
			TransformIndices.Num(), SourceTransformIndices.Num());
		return false;
	}

	CreateConvexHullAttributesIfNeeded(Collection);

	TManagedArrayAccessor<Chaos::FConvexPtr> ToConvexHullAttribute(Collection, ConvexHullAttributeName, ConvexGroupName);
	TManagedArrayAccessor<Chaos::FConvexPtr> FromConvexHullAttribute(SourceCollection, ConvexHullAttributeName, ConvexGroupName);
	TManagedArrayAccessor<TSet<int32>> ToTransformToConvexIndicesAttribute(Collection, TransformToConvexIndicesName, FTransformCollection::TransformGroup);
	TManagedArrayAccessor<TSet<int32>> FromTransformToConvexIndicesAttribute(SourceCollection, TransformToConvexIndicesName, FTransformCollection::TransformGroup);

	if (!FromConvexHullAttribute.IsValid() || !FromTransformToConvexIndicesAttribute.IsValid())
	{
		UE_LOGF(LogChaos, Warning,
			"Failed to copy convex hulls: Hull attributes not found on source collection");
		return false;
	}

	TManagedArray<TSet<int32>>& ToTransformToConvexIndices = ToTransformToConvexIndicesAttribute.Modify();
	TManagedArray<Chaos::FConvexPtr>& ToConvexHull = ToConvexHullAttribute.Modify();
	
	TSet<int32> ConvexToRemove;
	for (int32 ToTransferIdx = 0; ToTransferIdx < TransformIndices.Num(); ++ToTransferIdx)
	{
		int32 FromTransformIdx = SourceTransformIndices[ToTransferIdx];
		int32 ToTransformIdx = TransformIndices[ToTransferIdx];
		if (!ToTransformToConvexIndices.IsValidIndex(ToTransformIdx))
		{
			UE_LOGF(LogChaos, Warning,
				"Invalid transform index: %d",
				ToTransformIdx);
			continue;
		}
		if (!FromTransformToConvexIndicesAttribute.Get().IsValidIndex(FromTransformIdx))
		{
			UE_LOGF(LogChaos, Warning,
				"Invalid source transform index: %d",
				FromTransformIdx);
			continue;
		}
		const TSet<int32>& FromIndices = FromTransformToConvexIndicesAttribute[FromTransformIdx];

		if (bSkipIfEmpty && FromIndices.IsEmpty())
		{
			continue;
		}

		ConvexToRemove.Append(ToTransformToConvexIndices[ToTransformIdx]);
		TSet<int32>& ToConvexIndices = ToTransformToConvexIndices[ToTransformIdx];
		ToConvexIndices.Reset();
		
		int32 NewConvexIndexStart = Collection.AddElements(FromIndices.Num(), FGeometryCollection::ConvexGroup);
		for (int32 FromConvexIdx : FromIndices)
		{
			int32 ToConvexIdx = NewConvexIndexStart++;
			ToConvexIndices.Add(ToConvexIdx);
			ToConvexHull[ToConvexIdx] = FromConvexHullAttribute[FromConvexIdx];
		}
	}
	TArray<int32> ToRemoveArr = ConvexToRemove.Array();
	if (!ToRemoveArr.IsEmpty())
	{
		ToRemoveArr.Sort();
		FManagedArrayCollection::FProcessingParameters Params;
		Params.bDoValidation = false; // for perf reasons
		Collection.RemoveElements(FGeometryCollection::ConvexGroup, ToRemoveArr, Params);
	}

	return true;
}


void FGeometryCollectionConvexUtility::RemoveConvexHulls(FManagedArrayCollection* GeometryCollection, const TArray<int32>& TransformsToClearHullsFrom)
{
	if (GeometryCollection->HasGroup(FGeometryCollection::ConvexGroup) && GeometryCollection->HasAttribute("TransformToConvexIndices", FTransformCollection::TransformGroup))
	{
		TManagedArray<TSet<int32>>& TransformToConvexIndices = GeometryCollection->ModifyAttribute<TSet<int32>>("TransformToConvexIndices", FTransformCollection::TransformGroup);
		TArray<int32> ConvexIndices;
		for (int32 TransformIdx : TransformsToClearHullsFrom)
		{
			if (TransformToConvexIndices[TransformIdx].Num() > 0)
			{
				for (int32 ConvexIdx : TransformToConvexIndices[TransformIdx])
				{
					ConvexIndices.Add(ConvexIdx);
				}
				TransformToConvexIndices[TransformIdx].Empty();
			}
		}

		if (ConvexIndices.Num())
		{
			ConvexIndices.Sort();
			FManagedArrayCollection::FProcessingParameters Params;
			Params.bDoValidation = false; // for perf reasons
			GeometryCollection->RemoveElements(FGeometryCollection::ConvexGroup, ConvexIndices, Params);
		}
	}
}

/** Delete the convex hulls that are null */
void FGeometryCollectionConvexUtility::RemoveEmptyConvexHulls(FManagedArrayCollection& Collection)
{
	const FName ConvexGroupName = FGeometryCollection::ConvexGroup;
	const FName ConvexAttributeName = FGeometryCollection::ConvexHullAttribute;

	TManagedArrayAccessor<Chaos::FConvexPtr> ConvexHullAttribute(Collection, ConvexAttributeName, ConvexGroupName);
	if (ConvexHullAttribute.IsValid())
	{
		TManagedArray<Chaos::FConvexPtr>& ConvexHull = ConvexHullAttribute.Modify();

		// clear all null and empty hulls
		TArray<int32> EmptyConvex;
		for (int32 ConvexIdx = 0; ConvexIdx < ConvexHull.Num(); ConvexIdx++)
		{
			if (ConvexHull[ConvexIdx].IsValid())
			{
				if (ConvexHull[ConvexIdx]->NumVertices() == 0)
				{
					ConvexHull[ConvexIdx].SafeRelease();
					EmptyConvex.Add(ConvexIdx);
				}
			}
			else // (!ConvexHull[ConvexIdx].IsValid())
			{
				EmptyConvex.Add(ConvexIdx);
			}
		}

		FManagedArrayCollection::FProcessingParameters Params;
		Params.bDoValidation = false; // for perf reasons
		Collection.RemoveElements(ConvexGroupName, EmptyConvex, Params);
	}
}

void FGeometryCollectionConvexUtility::SetDefaults(FGeometryCollection* GeometryCollection, FName Group, uint32 StartSize, uint32 NumElements)
{
}


TManagedArray<int32>* FGeometryCollectionConvexUtility::GetCustomConvexFlags(FGeometryCollection* GeometryCollection, bool bAddIfMissing)
{
	if (!GeometryCollection->HasAttribute("HasCustomConvex", FTransformCollection::TransformGroup))
	{
		if (!bAddIfMissing)
		{
			return nullptr;
		}
		return &GeometryCollection->AddAttribute<int32>("HasCustomConvex", FTransformCollection::TransformGroup);
	}
	return &GeometryCollection->ModifyAttribute<int32>("HasCustomConvex", FTransformCollection::TransformGroup);
}


void FGeometryCollectionConvexUtility::CopyChildConvexes(const FGeometryCollection* FromCollection, const TArrayView<const int32>& FromTransformIdx, FGeometryCollection* ToCollection, const TArrayView<const int32>& ToTransformIdx, bool bLeafOnly)
{
	const TManagedArray<TSet<int32>>* InTransformToConvexIndices = FromCollection->FindAttribute<TSet<int32>>("TransformToConvexIndices", FTransformCollection::TransformGroup);
	const TManagedArray<Chaos::FConvexPtr>* InConvexHull = FromCollection->FindAttribute<Chaos::FConvexPtr>(FGeometryCollection::ConvexHullAttribute, FGeometryCollection::ConvexGroup);
	TOptional<FGeometryCollectionConvexUtility::FGeometryCollectionConvexData> OutConvex = FGeometryCollectionConvexUtility::GetConvexHullDataIfPresent(ToCollection);
	TManagedArray<int32>* OutCustomFlags = GetCustomConvexFlags(ToCollection, true);

	if (!ensure(InTransformToConvexIndices != nullptr && InConvexHull != nullptr) || !ensure(OutConvex.IsSet())) // missing convex data on collection(s); cannot copy
	{
		return;
	}

	if (!ensure(FromTransformIdx.Num() == ToTransformIdx.Num())) // these arrays must be matched up
	{
		return;
	}

	int32 InNumBones = FromCollection->NumElements(FGeometryCollection::TransformGroup);
	int32 LeafType = FGeometryCollection::ESimulationTypes::FST_Rigid;
	TArray<TArray<int32>> Children;
	Children.SetNum(InNumBones);
	for (int InBone = 0; InBone < InNumBones; InBone++)
	{
		if (FromCollection->SimulationType[InBone] == FGeometryCollection::ESimulationTypes::FST_None)
		{
			continue; // skip embedded geo
		}

		int32 ParentBone = FromCollection->Parent[InBone];
		if (ParentBone != INDEX_NONE)
		{
			if (FromCollection->SimulationType[ParentBone] != LeafType)
			{
				Children[ParentBone].Add(InBone);
			}
			else
			{
				// parent is leaf (should only happen for embedded geo, which we've already skipped above)
				continue;
			}
		}
	}

	auto GetChildrenWithConvex = [&FromCollection, bLeafOnly, &InTransformToConvexIndices, LeafType, &Children](int32 Bone, TArray<int32>& OutChildren)
	{
		TArray<int32> ToExpand = Children[Bone];
		if (ToExpand.IsEmpty()) // if no children, fall back to copying the convex on the bone itself
		{
			ToExpand.Add(Bone);
		}
		while (ToExpand.Num() > 0)
		{
			int32 ToProcess = ToExpand.Pop();
			bool bIsLeaf = FromCollection->SimulationType[ToProcess] == LeafType;
			if (bIsLeaf || (!bLeafOnly && (*InTransformToConvexIndices)[ToProcess].Num() > 0))
			{
				OutChildren.Add(ToProcess);
			}
			else if (Children[ToProcess].Num() > 0)
			{
				ToExpand.Append(Children[ToProcess]);
			}
		}
	};

	TArray<FTransform> InGlobalTransformArray, OutGlobalTransformArray;
	GeometryCollectionAlgo::GlobalMatrices(FromCollection->Transform, FromCollection->Parent, InGlobalTransformArray);
	if (FromCollection == ToCollection)
	{
		OutGlobalTransformArray = InGlobalTransformArray;
	}
	else
	{
		GeometryCollectionAlgo::GlobalMatrices(ToCollection->Transform, ToCollection->Parent, OutGlobalTransformArray);
	}
	
	// build the new convex data in separate arrays (which support incremental add)
	TArray<Chaos::FConvexPtr> ConvexToAdd;
	TArray<TSet<int32>> OutTransformToConvexIndices;
	OutTransformToConvexIndices.SetNum(OutConvex->TransformToConvexIndices.Num());
	TSet<int32> ToRemove;

	for (int32 i = 0; i < ToTransformIdx.Num(); i++)
	{
		int32 InBone = FromTransformIdx[i];
		int32 OutBone = ToTransformIdx[i];
		TArray<int32> BonesWithHulls;
		GetChildrenWithConvex(InBone, BonesWithHulls);

		ToRemove.Add(OutBone);
		(*OutCustomFlags)[OutBone] = true;
		OutTransformToConvexIndices[OutBone].Reset();
		for (int32 InBoneWithHulls : BonesWithHulls)
		{
			CopyHulls(
				InGlobalTransformArray, *InConvexHull, *InTransformToConvexIndices, InBoneWithHulls,
				OutGlobalTransformArray, ConvexToAdd, OutTransformToConvexIndices, OutBone);
		}
	}

	TArray<int32> ToRemoveArr = ToRemove.Array();
	ToRemoveArr.Sort();
	if (ToRemoveArr.Num() > 0)
	{
		RemoveConvexHulls(ToCollection, ToRemoveArr);
	}

	int32 NewNumConvex = ToCollection->NumElements(FGeometryCollection::ConvexGroup);

	for (int32 OutBone : ToTransformIdx)
	{
		OutConvex->TransformToConvexIndices[OutBone] = MoveTemp(OutTransformToConvexIndices[OutBone]);
		for (int32& ConvexInd : OutConvex->TransformToConvexIndices[OutBone])
		{
			ConvexInd += NewNumConvex;
		}
	}
	
	ToCollection->Resize(NewNumConvex + ConvexToAdd.Num(), FGeometryCollection::ConvexGroup);
	for (int32 i = 0; i < ConvexToAdd.Num(); i++)
	{
		OutConvex->ConvexHull[NewNumConvex + i] = MoveTemp(ConvexToAdd[i]);
	}
}

static float GetRelativeSizeDimensionFromVolume(const float Volume)
{
	if (UseVolumeToComputeRelativeSize)
	{
		return Volume;
	}
	return FGenericPlatformMath::Pow(Volume, 1.0f / 3.0f);
}

void FGeometryCollectionConvexUtility::SetVolumeAttributes(FManagedArrayCollection* Collection)
{
	TManagedArray<float>& Volumes = Collection->AddAttribute<float>("Volume", FTransformCollection::TransformGroup);
	TManagedArray<float>& Sizes = Collection->AddAttribute<float>("Size", FTransformCollection::TransformGroup);

	GeometryCollection::Facades::FCollectionMeshFacade MeshFacade(*Collection);
	GeometryCollection::Facades::FCollectionTransformFacade TransformFacade(*Collection);
	const TManagedArray<int32>* FoundSimulationType = Collection->FindAttributeTyped<int32>("SimulationType", FTransformCollection::TransformGroup);
	if (!MeshFacade.IsValid() || !TransformFacade.IsValid() || !FoundSimulationType)
	{
		ensureMsgf(false, TEXT("Cannot compute Volume and Size attributes for Collection: Missing required attributes"));
		return;
	}

	const TManagedArray<int32>& SimulationType = *FoundSimulationType;
	const TManagedArray<int32>& TransformToGeometryIndex = MeshFacade.TransformToGeometryIndexAttribute.Get();
	const TManagedArray<int32>& Parent = *TransformFacade.GetParents();

	TArray<FTransform> Transforms;
	GeometryCollectionAlgo::GlobalMatrices(*TransformFacade.FindTransforms(), Parent, Transforms);

	TArray<float> GeoVolumes;
	GeoVolumes.SetNumZeroed(Collection->NumElements(FGeometryCollection::GeometryGroup));
	ParallelFor(GeoVolumes.Num(), [&](int32 GeoIdx)
	{
		int32 Bone = MeshFacade.TransformIndexAttribute[GeoIdx];
		if (SimulationType[Bone] == FGeometryCollection::ESimulationTypes::FST_Rigid)
		{
			float ComputedVolume = (float)ComputeGeometryVolume(Collection, GeoIdx, Transforms[Bone], 1.0);
			// if a mesh has inverted orientation the volume will be negative, but we take the abs() here so it's more usable as a size metric
			GeoVolumes[GeoIdx] = FMath::Abs(ComputedVolume);
		}
	});

	float MaxGeoVolume = 0.0f;
	TArray<int32> RecursiveOrder = GeometryCollectionAlgo::ComputeRecursiveOrder(*Collection);
	Volumes.Fill(0.0f);
	for (const int32 Bone : RecursiveOrder)
	{
		if (SimulationType[Bone] == FGeometryCollection::ESimulationTypes::FST_Rigid)
		{
			int32 GeoIdx = TransformToGeometryIndex[Bone];
			if (GeoIdx == INDEX_NONE)
			{
				Volumes[Bone] = 0.0f;
				continue;
			}
			else
			{
				const float GeoVolume = GeoVolumes[GeoIdx];
				MaxGeoVolume = FMath::Max(MaxGeoVolume, GeoVolume);
				Volumes[Bone] = GeoVolume;
			}
		}
		const int32 ParentIdx = Parent[Bone];
		if (ParentIdx != INDEX_NONE)
		{
			Volumes[ParentIdx] += Volumes[Bone];
		}
	}

	if (UseLargestClusterToComputeRelativeSize)
	{
		// just go over all the bones as the largest clusters will be naturally larger in volume than any of the children
		for (int32 BoneIdx = 0; BoneIdx < Volumes.Num(); BoneIdx++)
		{
			MaxGeoVolume = FMath::Max(MaxGeoVolume, Volumes[BoneIdx]);
		}
	}
	
	const float ReferenceSize = GetRelativeSizeDimensionFromVolume(MaxGeoVolume);
	const float OneOverReferenceSize = MaxGeoVolume > 0 ? (1.0f / ReferenceSize) : 1.0f;
	for (int32 BoneIdx = 0; BoneIdx < Volumes.Num(); BoneIdx++)
	{
		const float ActualSize = GetRelativeSizeDimensionFromVolume(Volumes[BoneIdx]);
		Sizes[BoneIdx] = ActualSize * OneOverReferenceSize;
	}
}

void FGeometryCollectionConvexUtility::ConvertImplicitToConvexArray(const Chaos::FImplicitObject& InImplicit, const FTransform& Transform, TArray<FTransformedConvex>& InOutConvex)
{
	const Chaos::EImplicitObjectType PackedType = InImplicit.GetType(); // Type includes scaling and instancing data
	const Chaos::EImplicitObjectType InnerType = Chaos::GetInnerType(InImplicit.GetType());

	// Unwrap the wrapper/aggregating shapes
	if (Chaos::IsScaled(PackedType))
	{
		ConvertScaledImplicitToConvexArray(InImplicit, Transform, Chaos::IsInstanced(PackedType), InOutConvex);
		return;
	}

	if (Chaos::IsInstanced(PackedType))
	{
		ConvertInstancedImplicitToConvexArray(InImplicit, Transform, InOutConvex);
		return;
	}

	if (InnerType == Chaos::ImplicitObjectType::Transformed)
	{
		const Chaos::TImplicitObjectTransformed<Chaos::FReal, 3>* Transformed = InImplicit.template GetObject<Chaos::TImplicitObjectTransformed<Chaos::FReal, 3>>();
		const Chaos::FRigidTransform3 ImplicitTransform(
			Transform.TransformPosition(Transformed->GetTransform().GetLocation()),
			Transform.GetRotation() * Transformed->GetTransform().GetRotation()
		);
		if (const Chaos::FImplicitObject* TransformedImplicit = Transformed->GetTransformedObject())
		{
			ConvertImplicitToConvexArray(*TransformedImplicit, ImplicitTransform, InOutConvex);
		}
		return;
	}

	if (InnerType == Chaos::ImplicitObjectType::Union)
	{
		const Chaos::FImplicitObjectUnion* Union = InImplicit.template GetObject<Chaos::FImplicitObjectUnion>();
		int32 UnionIdx = 0;
		for (const Chaos::FImplicitObjectPtr& UnionImplicit : Union->GetObjects())
		{
			if (UnionImplicit)
			{
				ConvertImplicitToConvexArray(*UnionImplicit, Transform, InOutConvex);
			}
		}
		return;
	}

	if (InnerType == Chaos::ImplicitObjectType::UnionClustered)
	{
		// unsupported - quiet exit
		return;
	}

	// If we get here, we have an actual shape to render
	switch (InnerType)
	{
	case Chaos::ImplicitObjectType::Sphere:
	{
		if (const Chaos::FSphere*Sphere = InImplicit.template GetObject<Chaos::FSphere>())
		{
			const FTransform SphereTransform(FQuat::Identity, FVector(Sphere->GetCenterf()), FVector(Sphere->GetRadiusf()));
			const TArray<Chaos::FConvex::FVec3Type> Vertices(IcoSphere_Subdiv1, IcoSphere_Subdiv1_Num);

			FTransformedConvex& TransformedConvex = InOutConvex.Emplace_GetRef();
			TransformedConvex.Convex = Chaos::FConvexPtr( new Chaos::FConvex(Vertices, 0));
			TransformedConvex.Transform = SphereTransform * Transform;
		}
		break;
	}
	case Chaos::ImplicitObjectType::Box:
	{
		if (const Chaos::TBox<Chaos::FReal, 3>*Box = InImplicit.template GetObject<Chaos::TBox<Chaos::FReal, 3>>())
		{
			Chaos::FVec3f A(Box->Min());
			Chaos::FVec3f B(Box->Max());
			TArray<Chaos::FConvex::FVec3Type> Vertices;
			Vertices.Add({ A.X, A.Y, A.Z });
			Vertices.Add({ B.X, A.Y, A.Z });
			Vertices.Add({ A.X, B.Y, A.Z });
			Vertices.Add({ B.X, B.Y, A.Z });
			Vertices.Add({ A.X, A.Y, B.Z });
			Vertices.Add({ B.X, A.Y, B.Z });
			Vertices.Add({ A.X, B.Y, B.Z });
			Vertices.Add({ B.X, B.Y, B.Z });

			FTransformedConvex& TransformedConvex = InOutConvex.Emplace_GetRef();
			TransformedConvex.Convex = Chaos::FConvexPtr( new Chaos::FConvex(Vertices, 0));
			TransformedConvex.Transform = Transform;
		}
		break;
	}
	case Chaos::ImplicitObjectType::Capsule:
	{
		if (const Chaos::FCapsule* Capsule = InImplicit.template GetObject<Chaos::FCapsule>())
		{
			const FTransform CapsuleTransform(FRotationMatrix::MakeFromZ(FVector3d(Capsule->GetAxisf())).ToQuat(), FVector3d(Capsule->GetCenterf()));

			const FVector3f HalfHeightOffset(0, 0, Capsule->GetHeightf() * 0.5f);

			TArray<Chaos::FConvex::FVec3Type> Vertices;
			for (int32 VtxIndex = 0; VtxIndex < IcoHemisphere_Subdiv1_Num; VtxIndex++)
			{
				const float CapsuleRadius = Capsule->GetRadiusf();
				Vertices.Add(IcoHemisphere_Subdiv1[VtxIndex] * CapsuleRadius + HalfHeightOffset); // top hemisphere
				Vertices.Add(IcoHemisphere_Subdiv1[VtxIndex] * -CapsuleRadius - HalfHeightOffset); // bottom hemisphere
			}

			FTransformedConvex& TransformedConvex = InOutConvex.Emplace_GetRef();
			TransformedConvex.Convex = Chaos::FConvexPtr( new Chaos::FConvex(Vertices, 0));
			TransformedConvex.Transform = CapsuleTransform * Transform;
		}
		break;
	}
	case Chaos::ImplicitObjectType::Convex:
	{
		if (const Chaos::FConvex* Convex = InImplicit.template GetObject<Chaos::FConvex>())
		{
			FTransformedConvex& TransformedConvex = InOutConvex.Emplace_GetRef();
			TransformedConvex.Convex = Chaos::FConvexPtr(Convex->RawCopyAsConvex());
			TransformedConvex.Transform = Transform;
		}
		break;
	}
	// Unsuported shape types
	case Chaos::ImplicitObjectType::Plane:
	case Chaos::ImplicitObjectType::LevelSet:
	case Chaos::ImplicitObjectType::TaperedCylinder:
	case Chaos::ImplicitObjectType::Cylinder:
	case Chaos::ImplicitObjectType::TriangleMesh:
	case Chaos::ImplicitObjectType::HeightField:
	default:
		break;
	}
}

void FGeometryCollectionConvexUtility::ConvertScaledImplicitToConvexArray(
	const Chaos::FImplicitObject& Implicit, 
	const FTransform& WorldSpaceTransform, bool bInstanced, 
	TArray<FTransformedConvex>& InOutConvex)
{
	const Chaos::EImplicitObjectType InnerType = Chaos::GetInnerType(Implicit.GetType());
	switch (InnerType)
	{
		// we only support scaled / instanced convex
	case Chaos::ImplicitObjectType::Convex:
	{
		Chaos::FRigidTransform3 ScaleTM{ Chaos::FRigidTransform3::Identity };
		const Chaos::FConvex* Convex = nullptr;
		if (bInstanced)
		{
			if (const auto ScaledInstancedConvex = Implicit.template GetObject<Chaos::TImplicitObjectScaled<Chaos::FConvex, true>>())
			{
				ScaleTM.SetScale3D(ScaledInstancedConvex->GetScale());
				Convex = ScaledInstancedConvex->GetUnscaledObject();
			}
		}
		else
		{
			if (const auto ScaledConvex = Implicit.template GetObject<Chaos::TImplicitObjectScaled<Chaos::FConvex, false>>())
			{
				ScaleTM.SetScale3D(ScaledConvex->GetScale());
				Convex = ScaledConvex->GetUnscaledObject();
			}
		}
		if (Convex)
		{
			FTransformedConvex& TransformedConvex = InOutConvex.Emplace_GetRef();
			TransformedConvex.Convex = Chaos::FConvexPtr(Convex->RawCopyAsConvex());
			TransformedConvex.Transform = ScaleTM * WorldSpaceTransform;
		}
		break;
	}
	// unsupported types for scaled implicit
	case Chaos::ImplicitObjectType::Sphere:
	case Chaos::ImplicitObjectType::Box:
	case Chaos::ImplicitObjectType::Plane:
	case Chaos::ImplicitObjectType::Capsule:
	case Chaos::ImplicitObjectType::Transformed:
	case Chaos::ImplicitObjectType::Union:
	case Chaos::ImplicitObjectType::LevelSet:
	case Chaos::ImplicitObjectType::Unknown:
	case Chaos::ImplicitObjectType::TaperedCylinder:
	case Chaos::ImplicitObjectType::Cylinder:
	case Chaos::ImplicitObjectType::TriangleMesh:
	case Chaos::ImplicitObjectType::HeightField:
	default:
		break;
	}
}

void FGeometryCollectionConvexUtility::ConvertInstancedImplicitToConvexArray(
	const Chaos::FImplicitObject& Implicit, 
	const FTransform& Transform, 
	TArray<FTransformedConvex>& InOutConvex)
{
	const Chaos::EImplicitObjectType InnerType = Chaos::GetInnerType(Implicit.GetType());
	switch (InnerType)
	{
		// we only support instanced convex
	case Chaos::ImplicitObjectType::Convex:
	{
		const Chaos::TImplicitObjectInstanced<Chaos::FConvex>* Instanced = Implicit.template GetObject< Chaos::TImplicitObjectInstanced< Chaos::FConvex>>();
		if (const Chaos::FConvex* Convex = Instanced->GetInstancedObject())
		{
			FTransformedConvex& TransformedConvex = InOutConvex.Emplace_GetRef();
			TransformedConvex.Convex = Chaos::FConvexPtr(Convex->RawCopyAsConvex());
			TransformedConvex.Transform = Transform;
		}
		break;
	}

	// unsupported types for scaled implicit
	case Chaos::ImplicitObjectType::Sphere:
	case Chaos::ImplicitObjectType::Box:
	case Chaos::ImplicitObjectType::Plane:
	case Chaos::ImplicitObjectType::Capsule:
	case Chaos::ImplicitObjectType::Transformed:
	case Chaos::ImplicitObjectType::Union:
	case Chaos::ImplicitObjectType::LevelSet:
	case Chaos::ImplicitObjectType::Unknown:
	case Chaos::ImplicitObjectType::TaperedCylinder:
	case Chaos::ImplicitObjectType::Cylinder:
	case Chaos::ImplicitObjectType::TriangleMesh:
	case Chaos::ImplicitObjectType::HeightField:
	default:
		break;
	}
}


// local (static) helpers for the below Convex Utility code
namespace
{
	// Helpful struct to represent a convex hull with a local editable representation, to perform intersections / clipping on it
	struct FHullPolygons
	{
		// simple packed representation for convex hull faces, where each polygon's indices are listed sequentially,
		// and negative values indicate the number of vertices in the next polygon. If no negative value is listed, polygon is a triangle.
		TArray<int32> PackedPolygons;

		// Copy of hull vertices, to be refined through plane cuts
		TArray<Chaos::FVec3f> Vertices;
		Chaos::FAABB3f Bounds;

		void Reset()
		{
			Vertices.Reset();
			PackedPolygons.Reset();
			Bounds = Chaos::FAABB3f::EmptyAABB();
		}

		FHullPolygons() = default;

		FHullPolygons(const Chaos::FConvex& HullIn)
		{
			Init(HullIn);
		}

		void Init(const Chaos::FConvex& HullIn)
		{
			Reset();
			Vertices = HullIn.GetVertices();
			const Chaos::FConvexStructureData& HullData = HullIn.GetStructureData();
			int32 NumPlanes = HullIn.NumPlanes();
			PackedPolygons.Reserve(NumPlanes * 3);
			for (int PlaneIdx = 0; PlaneIdx < NumPlanes; PlaneIdx++)
			{
				int32 NumPlaneVerts = HullData.NumPlaneVertices(PlaneIdx);
				if (NumPlaneVerts > 3)
				{
					PackedPolygons.Add(-NumPlaneVerts);
				}
				for (int32 PlaneVertexIdx = 0; PlaneVertexIdx < NumPlaneVerts; PlaneVertexIdx++)
				{
					PackedPolygons.Add(HullData.GetPlaneVertex(PlaneIdx, PlaneVertexIdx));
				}
			}
			Bounds = HullIn.GetLocalBoundingBox();
		}

		void Intersect(const Chaos::FConvex& OtherHull, float ExpandAmount)
		{
			// Arrays to store intermediate plane cut data
			TArray<int32> NewPolygons;
			TMap<FIntVector2, int> NewVertices; // mapping from edges to new vertices
			TArray<float> SignedDist; // signed distance from vertices to a cutting plane
			TArray<int32> VertexRemap;
			TMap<int32, int32> OpenEdgeVertMap;

			// Cut by the plane through PlanePt with PlaneNormal. Note: Must pre-apply ExpandAmount; we do not assume the plane is shifted here.
			// Uses the intermediate data above as temp storage
			auto PlaneCut = [&NewPolygons, &NewVertices, &SignedDist, &VertexRemap, &OpenEdgeVertMap, this](Chaos::FVec3f PlanePt, Chaos::FVec3f PlaneNormal)
			{
				NewPolygons.Reset(PackedPolygons.Num());
				NewVertices.Reset();
				OpenEdgeVertMap.Reset();
				SignedDist.SetNum(Vertices.Num(), EAllowShrinking::No);
				VertexRemap.SetNum(Vertices.Num(), EAllowShrinking::No);
				int32 OpenEdgeStart = -1;

				// Possible optimization: if many vertices, check plane vs AABB corners first?

				int32 OutTotal = 0;
				for (int32 VertIdx = 0; VertIdx < Vertices.Num(); ++VertIdx)
				{
					float SD = (float)(Vertices[VertIdx] - PlanePt).Dot(PlaneNormal);
					SignedDist[VertIdx] = SD;
					OutTotal += int32(SD > 0);
				}
				// hull is fully outside plane's half-space
				if (OutTotal == Vertices.Num())
				{
					Vertices.Empty();
					PackedPolygons.Empty();
					return;
				}
				// hull is fully inside plane's half-space
				if (OutTotal == 0)
				{
					return;
				}

				for (int32 Idx = 0, PolyLen = 3; Idx < PackedPolygons.Num(); Idx += PolyLen)
				{
					// extract length of current polygon
					PolyLen = 3;
					int32 OrigStart = Idx;
					if (PackedPolygons[Idx] < 0)
					{
						PolyLen = -PackedPolygons[Idx];
						Idx++;
					}
					int32 Start = Idx;

					// helper to convert index within polygon to index within vertices array
					auto ToV = [this, Start, PolyLen](int32 SubIdx) -> int32
					{
						checkSlow(SubIdx >= 0 && SubIdx < PolyLen);
						int32 VertIdx = PackedPolygons[Start + SubIdx];
						checkSlow(VertIdx >= 0);
						return VertIdx;
					};
					// track where the polygon crosses the plane to decide what to do with it
					int32 OutCount = 0;
					int32 FirstIn = -1, FirstOut = -1, LastIn = -1;
					for (int32 SubIdx = 0; SubIdx < PolyLen; ++SubIdx)
					{
						float SD = SignedDist[ToV(SubIdx)];
						bool IsOut = SD > 0;
						if (FirstIn == -1)
						{
							if (!IsOut)
							{
								FirstIn = SubIdx;
							}
						}
						else if (FirstOut == -1 && IsOut)
						{
							LastIn = SubIdx - 1;
							FirstOut = SubIdx;
						}
						OutCount += int32(IsOut);
					}
					if (FirstOut == -1)
					{
						FirstOut = 0;
						LastIn = PolyLen - 1;
					}
					if (OutCount == PolyLen)
					{
						continue;
					}
					if (OutCount == 0)
					{
						// copy original polygon data
						for (int32 CopyIdx = OrigStart; CopyIdx < Start + PolyLen; ++CopyIdx)
						{
							NewPolygons.Add(PackedPolygons[CopyIdx]);
						}
						continue;
					}
					int32 NewPolyLen = LastIn + 1 - FirstIn;
					if (FirstIn == 0)
					{
						int32 WalkBack = PolyLen - 1;
						while (WalkBack > 0 && SignedDist[ToV(WalkBack)] <= 0)
						{
							FirstIn = WalkBack--;
							NewPolyLen++;
						}
					}
					auto GetCrossVertIdx = [&NewVertices, &SignedDist, this](int32 InsideVertIdx, int32 OutsideVertIdx) -> int32
					{
						checkSlow(InsideVertIdx != OutsideVertIdx);
						// If within zero tolerance of plane, snap to plane
						float InsideSD = SignedDist[InsideVertIdx];
						checkSlow(InsideSD <= 0);
						if (InsideSD > -FMathf::ZeroTolerance)
						{
							return -1;
						}
						FIntVector2 Key(InsideVertIdx, OutsideVertIdx);
						int32* FoundVert = NewVertices.Find(Key);
						if (!FoundVert)
						{
							float OutsideSD = SignedDist[OutsideVertIdx];
							checkSlow(OutsideSD >= 0);
							Chaos::FVec3f NewVert = FMath::Lerp(Vertices[InsideVertIdx], Vertices[OutsideVertIdx], InsideSD / (InsideSD - OutsideSD));
							int32 NewVertIdx = Vertices.Add(NewVert);
							NewVertices.Add(Key, NewVertIdx);
							return NewVertIdx;
						}
						return *FoundVert;
					};
					int32 FirstCross = GetCrossVertIdx(ToV(FirstIn), ToV((FirstIn + PolyLen - 1) % PolyLen));
					int32 LastCross = GetCrossVertIdx(ToV(LastIn), ToV(FirstOut));
					int32 OpenPlaneEdgeVA = FirstCross;
					int32 OpenPlaneEdgeVB = LastCross;
					if (FirstCross != -1)
					{
						NewPolyLen++;
					}
					else
					{
						OpenPlaneEdgeVA = ToV(FirstIn);
					}
					if (LastCross != -1)
					{
						NewPolyLen++;
					}
					else
					{
						OpenPlaneEdgeVB = ToV(LastIn);
					}

					if (NewPolyLen < 2) // single co-incident vertex; no open edge here
					{
						continue;
					}
					OpenEdgeStart = OpenPlaneEdgeVB;
					OpenEdgeVertMap.Add(OpenPlaneEdgeVA, OpenPlaneEdgeVB);
					if (NewPolyLen == 2)
					{
						continue;
					}
					if (NewPolyLen > 3)
					{
						NewPolygons.Add(-NewPolyLen);
					}
					int32 NewPolygonStart = NewPolygons.Num();
					if (FirstCross != -1)
					{
						NewPolygons.Add(FirstCross);
					}
					int32 AddStart = FirstIn;
					if (FirstIn > LastIn)
					{
						for (int32 SubIdx = FirstIn; SubIdx < PolyLen; ++SubIdx)
						{
							NewPolygons.Add(ToV(SubIdx));
						}
						AddStart = 0;
					}
					for (int32 SubIdx = AddStart; SubIdx <= LastIn; ++SubIdx)
					{
						NewPolygons.Add(ToV(SubIdx));
					}
					if (LastCross != -1)
					{
						NewPolygons.Add(LastCross);
					}

					check(NewPolygons.Num() - NewPolygonStart == NewPolyLen);
				}

				// add the closing polygon
				if (OpenEdgeStart != -1 && OpenEdgeVertMap.Num() > 2)
				{
					int32 OrigEnd = NewPolygons.Num();
					int32 PolyEdges = OpenEdgeVertMap.Num();
					if (PolyEdges > 3)
					{
						NewPolygons.Add(-PolyEdges);
					}
					int32 TraverseIdx = OpenEdgeStart;
					int32 Added = 0;
					do
					{
						NewPolygons.Add(TraverseIdx);
						int32* FoundNext = OpenEdgeVertMap.Find(TraverseIdx);
						if (!FoundNext)
						{
							break;
						}
						TraverseIdx = *FoundNext;
						Added++;
					} while (TraverseIdx != OpenEdgeStart && Added < PolyEdges);
					if (Added != PolyEdges || TraverseIdx != OpenEdgeStart)
					{
						// failsafe if we didn't find a closed loop covering all edges:
						// add a triangle fan closing off the edges that we did find
						NewPolygons.SetNum(OrigEnd, EAllowShrinking::No);
						Chaos::FVec3f Center(0, 0, 0);
						float CenterWt = 0;
						int32 CenterIdx = Vertices.Num();
						for (TPair<int32, int32> KV : OpenEdgeVertMap)
						{
							NewPolygons.Add(KV.Key);
							NewPolygons.Add(KV.Value);
							NewPolygons.Add(CenterIdx);
							Center += Vertices[KV.Key];
							CenterWt += 1.f;
						}
						Center /= CenterWt;
						Vertices.Add(Center);
					}
				}

				// NewPolygons now contains the updated polygon data
				Swap(PackedPolygons, NewPolygons);
				// Compress the vertex array to only include the vertices that weren't outside
				// and track how the indices were remapped
				int32 NumKept = 0;
				const int32 OldVertCount = SignedDist.Num();
				for (int32 OldV = 0; OldV < OldVertCount; ++OldV)
				{
					if (SignedDist[OldV] <= 0)
					{
						int32 UseNewV = NumKept++;
						VertexRemap[OldV] = UseNewV;
						checkSlow(OldV >= UseNewV);
						Vertices[UseNewV] = Vertices[OldV];
					}
				}
				// Translate back the new vertices
				if (NumKept < OldVertCount)
				{
					for (int32 OldIdx = OldVertCount, AddedIdx = 0; OldIdx < Vertices.Num(); ++OldIdx, ++AddedIdx)
					{
						Vertices[NumKept + AddedIdx] = Vertices[OldIdx];
					}
					int32 NumNew = Vertices.Num() - OldVertCount;
					Vertices.SetNum(NumKept + NumNew, EAllowShrinking::No);
				}
				// Update the polygons w/ the compressed vertex indices
				for (int32& VIdx : PackedPolygons)
				{
					if (VIdx >= 0) // Only remap vertices, not polygon sizes
					{
						if (VIdx < OldVertCount)
						{
							VIdx = VertexRemap[VIdx];
						}
						else // newly-created vertices are kept in the same order at the end of the array
						{
							VIdx = NumKept + (VIdx - OldVertCount);
						}
					}
				}
			};

			// TODO: For performance, consider also pre-cutting with (some of) OtherHull's (expanded) bounding box planes
			//Chaos::FConvex::FAABB3Type OtherBounds = OtherHull.GetLocalBoundingBox();
			//OtherBounds.Thicken(ExpandAmount);

			// Cut with each convex plane
			const int32 NumPlanes = OtherHull.NumPlanes();
			for (int32 PlaneIdx = 0; PlaneIdx < NumPlanes; ++PlaneIdx)
			{
				Chaos::TPlaneConcrete<float, 3> Plane = OtherHull.GetPlaneRaw(PlaneIdx);
				Chaos::FVec3f N = Plane.Normal();
				Chaos::FVec3f X = Plane.X();
				X += N * ExpandAmount;
				PlaneCut(X, N);
			}

			// When ExpandAmount is positive, also clip the hull at offsets of average edge planes for 'sharp' edges
			if (ExpandAmount > 0)
			{
				const int32 NumEdges = OtherHull.NumEdges();
				for (int32 EdgeIdx = 0; EdgeIdx < NumEdges; ++EdgeIdx)
				{
					Chaos::TPlaneConcrete<float, 3> EPlane0 = OtherHull.GetPlaneRaw(OtherHull.GetEdgePlane(EdgeIdx, 0));
					Chaos::TPlaneConcrete<float, 3> EPlane1 = OtherHull.GetPlaneRaw(OtherHull.GetEdgePlane(EdgeIdx, 1));
					float NormalDot = EPlane0.Normal().Dot(EPlane1.Normal());
					if (NormalDot < -.1) // add an extra plane when not doing so would leave ~1.5x more space than the expected offset across from the edge, due to the miter
					{
						Chaos::FVec3f AvgNormal = EPlane0.Normal() + EPlane1.Normal();
						if (AvgNormal.Normalize())
						{
							Chaos::FVec3f EdgeVert = OtherHull.GetVertex(OtherHull.GetEdgeVertex(EdgeIdx, 0));
							PlaneCut(EdgeVert + AvgNormal * ExpandAmount, AvgNormal);
						}
					}

				}
			}
		}

		float ComputeArea()
		{
			float Area = 0;
			for (int32 Idx = 0, PolyLen = 3; Idx < PackedPolygons.Num(); Idx += PolyLen)
			{
				// extract length of current polygon
				PolyLen = 3;
				if (PackedPolygons[Idx] < 0)
				{
					PolyLen = -PackedPolygons[Idx];
					Idx++;
				}
				int32 Start = Idx;

				// Add area of triangle fan covering the polygon
				Chaos::FVec3f V0 = Vertices[PackedPolygons[Start]];
				for (int32 SubIdx = 1; SubIdx + 1 < PolyLen; ++SubIdx)
				{
					Chaos::FVec3f V1 = Vertices[PackedPolygons[Start + SubIdx]];
					Chaos::FVec3f V2 = Vertices[PackedPolygons[Start + SubIdx + 1]];
					Area += UE::Geometry::VectorUtil::Area<float>(V0, V1, V2);
				}
			}
			return Area;
		}

		void EstimateSharpContact(const Chaos::FConvex* HullA, const Chaos::FConvex* HullB, float& OutSharpContact, float& OutMaxSharpContact)
		{
			UE::Geometry::FExtremePoints3f ExtremePts(Vertices.Num(), [this](int32 Idx) {return Vertices[Idx];});
			if (ExtremePts.Dimension < 1)
			{
				OutSharpContact = 0;
				OutMaxSharpContact = 1;
				return; // degenerate/empty contact
			}
			UE::Geometry::FInterval1f IntersectionIntervals[2];
			UE::Geometry::FInterval1f HullAIntervals[2], HullBIntervals[2];
			if (ExtremePts.Dimension > 1)
			{
				auto SetIntervals = [&ExtremePts](const TArray<Chaos::FVec3f>& UseVertices, UE::Geometry::FInterval1f* Intervals) -> void
				{
					for (FVector3f Vertex : UseVertices)
					{
						Intervals[0].Contain(Vertex.Dot(ExtremePts.Basis[1]));
						if (ExtremePts.Dimension > 2)
						{
							Intervals[1].Contain(Vertex.Dot(ExtremePts.Basis[2]));
						}
					}
				};
				SetIntervals(Vertices, IntersectionIntervals);
				SetIntervals(HullA->GetVertices(), HullAIntervals);
				SetIntervals(HullB->GetVertices(), HullBIntervals);
			}
			auto IntervalsMaxLen = [](UE::Geometry::FInterval1f* Intervals)
			{
				return FMath::Max(Intervals[0].Length(), Intervals[1].Length());
			};
			OutSharpContact = IntervalsMaxLen(IntersectionIntervals);
			OutMaxSharpContact = FMath::Min(IntervalsMaxLen(HullAIntervals), IntervalsMaxLen(HullBIntervals));
		}
	};

	static float ComputeHullArea(const Chaos::FConvex& Hull)
	{
		float Area = 0;
		const Chaos::FConvexStructureData& HullData = Hull.GetStructureData();
		int32 NumPlanes = Hull.NumPlanes();
		for (int PlaneIdx = 0; PlaneIdx < NumPlanes; PlaneIdx++)
		{
			int32 NumPlaneVerts = HullData.NumPlaneVertices(PlaneIdx);
			Chaos::FVec3f V0 = Hull.GetVertex(HullData.GetPlaneVertex(PlaneIdx, 0));
			for (int32 PlaneVertexIdx = 1; PlaneVertexIdx + 1 < NumPlaneVerts; PlaneVertexIdx++)
			{
				Chaos::FVec3f V1 = Hull.GetVertex(HullData.GetPlaneVertex(PlaneIdx, PlaneVertexIdx));
				Chaos::FVec3f V2 = Hull.GetVertex(HullData.GetPlaneVertex(PlaneIdx, PlaneVertexIdx + 1));
				Area += UE::Geometry::VectorUtil::Area<float>(V0, V1, V2);
			}
		}
		return Area;
	}
}



namespace UE::GeometryCollectionConvexUtility
{


	void HullIntersectionStats(const Chaos::FConvex* HullA, const Chaos::FConvex* HullB, float HullBExpansion, float& OutArea, float& OutMaxArea, float& OutSharpContact, float& OutMaxSharpContact)
	{
		FHullPolygons HullPolygons(*HullA);
		HullPolygons.Intersect(*HullB, HullBExpansion);
		OutArea = HullPolygons.ComputeArea();
		// The maximum intersection area is ~ the minimum of the two hull areas
		float MaxIntersectionArea = FMath::Min(ComputeHullArea(*HullA), ComputeHullArea(*HullB));
		OutMaxArea = MaxIntersectionArea;
		HullPolygons.EstimateSharpContact(HullA, HullB, OutSharpContact, OutMaxSharpContact);
	}

	void IntersectConvexHulls(Chaos::FConvex* ResultHull, const Chaos::FConvex* ClipHull, float ClipHullOffset, const Chaos::FConvex* UpdateHull, const FTransform* ClipHullTransform, const FTransform* UpdateHullTransform, const FTransform* ResultTransform, double SimplificationDistanceThreshold)
	{
		FHullPolygons HullPolygons(*ClipHull);
		bool bNeedRecomputeBounds = false;
		if (ClipHullTransform)
		{
			bNeedRecomputeBounds = true;
			for (Chaos::FVec3f& V : HullPolygons.Vertices)
			{
				V = (Chaos::FVec3f)ClipHullTransform->TransformPosition((FVector)V);
			}
		}
		if (UpdateHullTransform)
		{
			bNeedRecomputeBounds = true;
			for (Chaos::FVec3f& V : HullPolygons.Vertices)
			{
				V = (Chaos::FVec3f)UpdateHullTransform->InverseTransformPosition((FVector)V);
			}
		}
		if (bNeedRecomputeBounds)
		{
			HullPolygons.Bounds = Chaos::FAABB3f::EmptyAABB();
			for (Chaos::FVec3f& V : HullPolygons.Vertices)
			{
				HullPolygons.Bounds.GrowToInclude(V);
			}
		}
		HullPolygons.Intersect(*UpdateHull, ClipHullOffset);
		if (ResultTransform)
		{
			for (Chaos::FVec3f& V : HullPolygons.Vertices)
			{
				V = (Chaos::FVec3f)ResultTransform->TransformPosition((FVector)V);
			}
		}
		FilterHullPoints(HullPolygons.Vertices, SimplificationDistanceThreshold);
		*ResultHull = Chaos::FConvex(HullPolygons.Vertices, UpdateHull->GetMargin());
	}

	bool CHAOS_API GetExistingConvexHullsInSharedSpace(const FManagedArrayCollection* Collection, FConvexHulls& OutConvexHulls, bool bLeafOnly)
	{
		if (!Collection->HasAttribute("TransformToConvexIndices", FTransformCollection::TransformGroup) ||
			!Collection->HasAttribute(FGeometryCollection::ConvexHullAttribute, FGeometryCollection::ConvexGroup))
		{
			return false;
		}
		
		const TManagedArray<TSet<int32>>& OrigTransformToConvexIndices = Collection->GetAttribute<TSet<int32>>("TransformToConvexIndices", FTransformCollection::TransformGroup);
		const TManagedArray<Chaos::FConvexPtr>& OrigConvexHulls = Collection->GetAttribute<Chaos::FConvexPtr>(FGeometryCollection::ConvexHullAttribute, FGeometryCollection::ConvexGroup);
		const TManagedArray<int32>* SimulationType = Collection->FindAttribute<int32>("SimulationType", FTransformCollection::TransformGroup);
		if (bLeafOnly && !SimulationType)
		{
			return false;
		}

		const int32 NumTransform = OrigTransformToConvexIndices.Num();
		OutConvexHulls.TransformToHullsIndices.SetNum(NumTransform);

		GeometryCollection::Facades::FCollectionTransformFacade TransformFacade(*Collection);
		TArray<FTransform> GlobalTransformArray = TransformFacade.ComputeCollectionSpaceTransforms();
		
		TArray<int32> NewConvexToTransformIndices;
		TArray<int32> NewConvexToOrigConvexIndices;
		NewConvexToTransformIndices.Reserve(OrigConvexHulls.Num());
		NewConvexToOrigConvexIndices.Reserve(OrigConvexHulls.Num());
		int32 NumNewConvex = 0;
		for (int32 TransformIdx = 0; TransformIdx < OrigTransformToConvexIndices.Num(); ++TransformIdx)
		{
			if (bLeafOnly && (*SimulationType)[TransformIdx] != FGeometryCollection::ESimulationTypes::FST_Rigid)
			{
				continue;
			}
			for (int32 OrigConvexIdx : OrigTransformToConvexIndices[TransformIdx])
			{
				int32 NewConvexIdx = NumNewConvex++;
				NewConvexToTransformIndices.Add(TransformIdx);
				NewConvexToOrigConvexIndices.Add(OrigConvexIdx);
				checkSlow(NewConvexToTransformIndices.Num() == NumNewConvex);
				checkSlow(NewConvexToOrigConvexIndices.Num() == NumNewConvex);
				OutConvexHulls.TransformToHullsIndices[TransformIdx].Add(NewConvexIdx);
			}
		}

		OutConvexHulls.Hulls.SetNum(NumNewConvex);
		OutConvexHulls.OverlapRemovalShrinkPercent = 0;
		OutConvexHulls.Pivots.Reset(); // Note: No scaling is applied so pivots are not used

		ParallelFor(NewConvexToOrigConvexIndices.Num(), [&](int32 NewConvexIdx)
		{
			int32 TransformIdx = NewConvexToTransformIndices[NewConvexIdx];
			FTransform Transform = GlobalTransformArray[TransformIdx];
			int32 OrigConvexIdx = NewConvexToOrigConvexIndices[NewConvexIdx];
			TArray<Chaos::FConvex::FVec3Type> HullPts;
			HullPts.Reserve(OrigConvexHulls[OrigConvexIdx]->GetVertices().Num());
			for (const Chaos::FConvex::FVec3Type& P : OrigConvexHulls[OrigConvexIdx]->GetVertices())
			{
				FVector PVec(P);
				HullPts.Add((Chaos::FConvex::FVec3Type)(Transform.TransformPosition(PVec)));
			}
			OutConvexHulls.Hulls[NewConvexIdx] = new Chaos::FConvex(HullPts, UE_KINDA_SMALL_NUMBER);
		});

		return true;
	}
}
