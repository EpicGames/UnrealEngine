// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionProximityPropertiesInterface.h"
#include "GeometryCollection/GeometryCollection.h"

const FName FGeometryCollectionProximityPropertiesInterface::ProximityPropertiesGroup = "ProximityProperties";
const FName FGeometryCollectionProximityPropertiesInterface::ProximityDistanceThreshold = "DistanceThreshold";
const FName FGeometryCollectionProximityPropertiesInterface::ProximityDetectionMethod = "DetectionMethod";
const FName FGeometryCollectionProximityPropertiesInterface::ProximityAsConnectionGraph = "AsConnectionGraph";
const FName FGeometryCollectionProximityPropertiesInterface::ProximityConnectionGraphContactAreaMethod = "ConnectionGraphContactAmount";
const FName FGeometryCollectionProximityPropertiesInterface::ProximityRequireContactAmount = "RequireContactAmount";
const FName FGeometryCollectionProximityPropertiesInterface::ProximityContactMethod = "ContactMethod";

FGeometryCollectionProximityPropertiesInterface::FGeometryCollectionProximityPropertiesInterface(FGeometryCollection* InGeometryCollection)
	: FManagedArrayInterface(InGeometryCollection)
{}

void 
FGeometryCollectionProximityPropertiesInterface::InitializeInterface()
{
	if (!ManagedCollection->HasGroup(ProximityPropertiesGroup))
	{
		ManagedCollection->AddGroup(ProximityPropertiesGroup);
	}

	if (!ManagedCollection->HasAttribute(ProximityDetectionMethod, ProximityPropertiesGroup))
	{
		ManagedCollection->AddAttribute<int32>(ProximityDetectionMethod, ProximityPropertiesGroup);
	}

	if (!ManagedCollection->HasAttribute(ProximityDistanceThreshold, ProximityPropertiesGroup))
	{
		ManagedCollection->AddAttribute<float>(ProximityDistanceThreshold, ProximityPropertiesGroup);
	}

	if (!ManagedCollection->HasAttribute(ProximityAsConnectionGraph, ProximityPropertiesGroup))
	{
		ManagedCollection->AddAttribute<bool>(ProximityAsConnectionGraph, ProximityPropertiesGroup);
	}

	if (!ManagedCollection->HasAttribute(ProximityContactMethod, ProximityPropertiesGroup))
	{
		ManagedCollection->AddAttribute<uint8>(ProximityContactMethod, ProximityPropertiesGroup);
	}

	if (!ManagedCollection->HasAttribute(ProximityRequireContactAmount, ProximityPropertiesGroup))
	{
		ManagedCollection->AddAttribute<float>(ProximityRequireContactAmount, ProximityPropertiesGroup);
	}

	if (!ManagedCollection->HasAttribute(ProximityConnectionGraphContactAreaMethod, ProximityPropertiesGroup))
	{
		ManagedCollection->AddAttribute<uint8>(ProximityConnectionGraphContactAreaMethod, ProximityPropertiesGroup);
	}
}

void 
FGeometryCollectionProximityPropertiesInterface::CleanInterfaceForCook()
{
	RemoveInterfaceAttributes();
}

void 
FGeometryCollectionProximityPropertiesInterface::RemoveInterfaceAttributes()
{
	ManagedCollection->RemoveGroup(ProximityPropertiesGroup);
}

FGeometryCollectionProximityPropertiesInterface::FProximityProperties
FGeometryCollectionProximityPropertiesInterface::GetProximityProperties() const
{
	FProximityProperties Properties;
	const bool bHasProximityProperties = ManagedCollection->NumElements(ProximityPropertiesGroup) > 0;
	if (bHasProximityProperties)
	{
		const TManagedArray<bool>& AsConnectionGraph = ManagedCollection->GetAttribute<bool>(ProximityAsConnectionGraph, ProximityPropertiesGroup);
		const TManagedArray<uint8>& ConnectionContactMethod = ManagedCollection->GetAttribute<uint8>(ProximityConnectionGraphContactAreaMethod, ProximityPropertiesGroup);
		const TManagedArray<int32>& DetectionMethod = ManagedCollection->GetAttribute<int32>(ProximityDetectionMethod, ProximityPropertiesGroup);
		const TManagedArray<float>& DistanceThreshold = ManagedCollection->GetAttribute<float>(ProximityDistanceThreshold, ProximityPropertiesGroup);
		const TManagedArray<uint8>& ContactMethod = ManagedCollection->GetAttribute<uint8>(ProximityContactMethod, ProximityPropertiesGroup);
		const TManagedArray<float>& ContactAmount = ManagedCollection->GetAttribute<float>(ProximityRequireContactAmount, ProximityPropertiesGroup);

		constexpr int32 DefaultIndex = 0;
		Properties.bUseAsConnectionGraph = AsConnectionGraph[DefaultIndex];
		Properties.ContactAreaMethod = (EConnectionContactMethod)ConnectionContactMethod[DefaultIndex];
		Properties.DistanceThreshold = DistanceThreshold[DefaultIndex];
		Properties.Method = (EProximityMethod)DetectionMethod[DefaultIndex];
		Properties.ContactMethod = (EProximityContactMethod)ContactMethod[DefaultIndex];
		Properties.RequireContactAmount = ContactAmount[DefaultIndex];
	}
	return Properties;
}

void
FGeometryCollectionProximityPropertiesInterface::SetProximityProperties(const FProximityProperties& InProximityAttributes)
{
	int32 AttributeIndex = 0;
	const bool bHasProximityProperties = ManagedCollection->NumElements(ProximityPropertiesGroup) > 0;
	if (!bHasProximityProperties)
	{
		if (!ensure(ManagedCollection->HasGroup(ProximityPropertiesGroup)))
		{
			InitializeInterface();
		}
		AttributeIndex = ManagedCollection->AddElements(1, ProximityPropertiesGroup);
	}

	TManagedArray<bool>& AsConnectionGraph = ManagedCollection->ModifyAttribute<bool>(ProximityAsConnectionGraph, ProximityPropertiesGroup);
	TManagedArray<uint8>& ConnectionContactMethod = ManagedCollection->ModifyAttribute<uint8>(ProximityConnectionGraphContactAreaMethod, ProximityPropertiesGroup);
	TManagedArray<int32>& Method = ManagedCollection->ModifyAttribute<int32>(ProximityDetectionMethod, ProximityPropertiesGroup);
	TManagedArray<float>& DistanceThreshold = ManagedCollection->ModifyAttribute<float>(ProximityDistanceThreshold, ProximityPropertiesGroup);
	TManagedArray<uint8>& ContactMethod = ManagedCollection->ModifyAttribute<uint8>(ProximityContactMethod, ProximityPropertiesGroup);
	TManagedArray<float>& ContactAmount = ManagedCollection->ModifyAttribute<float>(ProximityRequireContactAmount, ProximityPropertiesGroup);

	AsConnectionGraph[AttributeIndex] = InProximityAttributes.bUseAsConnectionGraph;
	ConnectionContactMethod[AttributeIndex] = (uint8)InProximityAttributes.ContactAreaMethod;
	DistanceThreshold[AttributeIndex] = InProximityAttributes.DistanceThreshold;
	Method[AttributeIndex] = (int32)InProximityAttributes.Method;
	ContactMethod[AttributeIndex] = (uint8)InProximityAttributes.ContactMethod;
	ContactAmount[AttributeIndex] = InProximityAttributes.RequireContactAmount;
}
