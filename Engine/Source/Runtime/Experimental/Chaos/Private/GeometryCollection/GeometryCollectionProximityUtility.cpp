// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionProximityUtility.h"

#include "GeometryCollection/Facades/CollectionConnectionGraphFacade.h"
#include "GeometryCollection/Facades/CollectionHierarchyFacade.h"
#include "GeometryCollection/GeometryCollectionClusteringUtility.h"
#include "GeometryCollection/GeometryCollectionConvexUtility.h"
#include "Chaos/Convex.h"
#include "Chaos/GJK.h"
#include "CompGeom/ConvexHull3.h"
#include "VectorUtil.h"

#include "Spatial/SparseDynamicOctree3.h"
#include "Async/ParallelFor.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"


#include "Spatial/PointHashGrid3.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionProximityUtility)

DEFINE_LOG_CATEGORY_STATIC(LogChaosProximity, Verbose, All);

namespace UE { namespace GeometryCollectionInternal	{

static inline bool AreNormalsOpposite(const FVector3f& Normal0, const FVector3f& Normal1)
{
	return FVector3f::DotProduct(Normal0, Normal1) < (-1.0f + UE_KINDA_SMALL_NUMBER);
}

static inline bool TriOutsideEdge(const TStaticArray<FVector2f, 3>& Points, const FVector2f& B, const FVector2f& C, float Normal)
{
	const FVector2f CBPerp = UE::Geometry::PerpCW(C - B) * Normal;
	const FVector2f P0B = Points[0] - B;
	const FVector2f P1B = Points[1] - B;
	const FVector2f P2B = Points[2] - B;
	// Note: If we change this comparison to use a tolerance instead of 0, need to normalize CBPerp and also filter out degenerate tris
	bool bAnyInside =
		CBPerp.Dot(P0B) < 0 ||
		CBPerp.Dot(P1B) < 0 ||
		CBPerp.Dot(P2B) < 0;
	return !bAnyInside;
}

static inline bool TrianglesIntersect(const TStaticArray<FVector2f,3>& T0, const TStaticArray<FVector2f,3>& T1)
{
	// Test if one of the triangles has a side with all of the other triangle's points on the outside.

	// Orientation of T0 and T1, as sign of area (computed as sign of the Z component of a cross product of the triangle edges)
	float Normal0 = FMathf::Sign(T0[1].X - T0[0].X) * (T0[2].Y - T0[0].Y) - (T0[1].Y - T0[0].Y) * (T0[2].X - T0[0].X);
	float Normal1 = FMathf::Sign(T1[1].X - T1[0].X) * (T1[2].Y - T1[0].Y) - (T1[1].Y - T1[0].Y) * (T1[2].X - T1[0].X);
	
	// Triangles overlap if there is no edge that the other tri is completely on the 'outside' side of
	return !(
		TriOutsideEdge(T1, T0[0], T0[1], Normal0) ||
		TriOutsideEdge(T1, T0[1], T0[2], Normal0) ||
		TriOutsideEdge(T1, T0[2], T0[0], Normal0) ||
		TriOutsideEdge(T0, T1[0], T1[1], Normal1) ||
		TriOutsideEdge(T0, T1[1], T1[2], Normal1) ||
		TriOutsideEdge(T0, T1[2], T1[0], Normal1)
		);
}

// We'll bin triangles normals based on 20 dodecahedron directions, to filter which ones to test for overlap
struct FBinNormals
{
	static constexpr int NumBins = 20;

	TArray<FVector3f, TFixedAllocator<NumBins>> Bins;

	FBinNormals()
	{
		// We quantize surface normals into 20 uniform bins on a unit sphere surface, ie an icosahedron
		Bins.SetNum(NumBins);

		Bins[0] = FVector3f(0.171535f, -0.793715f, 0.583717f);
		Bins[1] = FVector3f(0.627078f, -0.778267f, 0.034524f);
		Bins[2] = FVector3f(0.491358f, 0.810104f, -0.319894f);
		Bins[3] = FVector3f(0.445554f, 0.804788f, 0.392214f);
		Bins[4] = FVector3f(0.245658f, -0.785111f, -0.568669f);
		Bins[5] = FVector3f(0.984880f, -0.161432f, 0.062144f);
		Bins[6] = FVector3f(0.247864f, -0.186425f, 0.950708f);
		Bins[7] = FVector3f(0.824669f, 0.212942f, -0.523975f);
		Bins[8] = FVector3f(0.750546f, 0.204339f, 0.628411f);
		Bins[9] = FVector3f(0.367791f, -0.172505f, -0.913787f);
		Bins[10] = -Bins[0];
		Bins[11] = -Bins[1];
		Bins[12] = -Bins[2];
		Bins[13] = -Bins[3];
		Bins[14] = -Bins[4];
		Bins[15] = -Bins[5];
		Bins[16] = -Bins[6];
		Bins[17] = -Bins[7];
		Bins[18] = -Bins[8];
		Bins[19] = -Bins[9];
	}

	// Note: a normal could point at an edge or corner of the dodecahedron, resulting in an ambiguous case that misses matching normals ...
	// Currently we just hope that the other proximity tests (vertex proximity and other faces) will find the proximity in these cases.
	// TODO: Consider testing multiple bins in ambiguous cases.
	int32 FindBestBin(const FVector3f& SurfaceNormal) const
	{
		// We select the bin with the highest alignment with the surface normal
		float BestAlignment = -1.0;
		int32 BestBin = INDEX_NONE;

		for (int32 BinIdx = 0; BinIdx < NumBins; ++BinIdx)
		{
			float Alignment = FVector3f::DotProduct(SurfaceNormal, Bins[BinIdx]);
			if (Alignment > BestAlignment)
			{
				BestAlignment = Alignment;
				BestBin = BinIdx;
			}
		}

		return BestBin;
	}
};

// Per-Geometry pre-computed spatial data used for computing 'precise' proximity information
struct FPerGeometrySpatial
{
	TArray<TArray<int32>, TFixedAllocator<FBinNormals::NumBins>> Bins;
	UE::Math::TBox<float> Bounds; // bounds in a shared space

	void InitBins(const FGeometryCollection* Collection, const FBinNormals& Binner, int32 GeoIdx, const TArray<FVector3f>& SurfaceNormals)
	{
		Bins.SetNum(FBinNormals::NumBins);
		int32 FaceStart = Collection->FaceStart[GeoIdx];
		int32 FaceEnd = FaceStart + Collection->FaceCount[GeoIdx];
		for (int32 FaceIdx = FaceStart; FaceIdx < FaceEnd; ++FaceIdx)
		{
			int32 BestBin = Binner.FindBestBin(SurfaceNormals[FaceIdx]);
			Bins[BestBin].Add(FaceIdx);
		}
	}

	// Note: Only contains mappings to geometry w/ *higher* indices
	TMap<int32, bool> CandidateContacts; // bool indicates if the contact has been confirmed
};

// Overall geometry-collection spatial data for 'Precise method' proximity detection
struct FGeometryCollectionProximitySpatial
{
	TArray<FVector3f> TransformedVertices;
	TArray<FVector3f> SurfaceNormals;
	TArray<FPerGeometrySpatial> GeoInfo;
	TArray<TSet<int32>> KnownProximity;
	UE::Math::TBox<float> OverallBounds;

	FGeometryCollectionProximitySpatial(const FGeometryCollection* Collection, float ProximityTolerance = KINDA_SMALL_NUMBER)
	{
		TransformVertices(Collection);
		GenerateSurfaceNormals(Collection);
		InitProximityFromVertices(Collection, ProximityTolerance);
		InitGeoNormalBins(Collection);
		InitCandidateContacts(Collection, ProximityTolerance);
		ComputeCoplanarContacts(Collection, ProximityTolerance);
	}

	void ComputeCoplanarContacts(const FGeometryCollection* Collection, float ProximityTolerance)
	{
		ParallelFor(GeoInfo.Num(), [this, &Collection, ProximityTolerance](int32 GeoIdx)
			{
				for (TPair<int32, bool>& PossibleContact : GeoInfo[GeoIdx].CandidateContacts)
				{
					if (PossibleContact.Value || KnownProximity[GeoIdx].Contains(PossibleContact.Key))
					{
						continue;
					}

					int32 OtherGeoIdx = PossibleContact.Key;
					for (int32 BinIdx = 0; BinIdx < FBinNormals::NumBins && !PossibleContact.Value; ++BinIdx)
					{
						int32 OtherBinIdx = (BinIdx + (FBinNormals::NumBins / 2)) % FBinNormals::NumBins;
						for (int32 FaceIdx : GeoInfo[GeoIdx].Bins[BinIdx])
						{
							// Skip if the face bounds don't overlap the geo bounds
							FIntVector3 Face = Collection->Indices[FaceIdx];
							UE::Math::TBox<float> FaceBox(EForceInit::ForceInit);
							FaceBox += TransformedVertices[Face.X];
							FaceBox += TransformedVertices[Face.Y];
							FaceBox += TransformedVertices[Face.Z];
							if (!GeoInfo[OtherGeoIdx].Bounds.Intersect(FaceBox.ExpandBy(ProximityTolerance)))
							{
								continue;
							}
							for (int32 OtherFaceIdx : GeoInfo[OtherGeoIdx].Bins[OtherBinIdx])
							{
								if (AreNormalsOpposite(SurfaceNormals[FaceIdx], SurfaceNormals[OtherFaceIdx]))
								{
									if (AreFacesCoPlanar(Collection, FaceIdx, OtherFaceIdx, ProximityTolerance))
									{
										if (DoFacesOverlap(Collection, FaceIdx, OtherFaceIdx))
										{
											PossibleContact.Value = true; // verified contact
											break; // don't need to consider the rest of the geometry
										}
									}
								}
							}
							if (PossibleContact.Value)
							{
								break;
							}
						}
					}
				}
			});
	}

	// Note: This function will destroy/move over the computed proximity data when building it in the output collection
	void MoveProximityToCollection(FGeometryCollection* Collection)
	{
		if (!Collection->HasAttribute("Proximity", FGeometryCollection::GeometryGroup))
		{
			const FManagedArrayCollection::FConstructionParameters GeometryDependency(FGeometryCollection::GeometryGroup);
			Collection->AddAttribute<TSet<int32>>("Proximity", FGeometryCollection::GeometryGroup, GeometryDependency);
		}

		TManagedArray<TSet<int32>>& Proximity = Collection->ModifyAttribute<TSet<int32>>("Proximity", FGeometryCollection::GeometryGroup);

		for (int32 GeoIdx = 0; GeoIdx < KnownProximity.Num(); ++GeoIdx)
		{
			Proximity[GeoIdx] = MoveTemp(KnownProximity[GeoIdx]);
		}
		for (int32 GeoIdx = 0; GeoIdx < KnownProximity.Num(); ++GeoIdx)
		{
			for (const TPair<int32, bool>& PossibleContact : GeoInfo[GeoIdx].CandidateContacts)
			{
				if (PossibleContact.Value)
				{
					Proximity[GeoIdx].Add(PossibleContact.Key);
					Proximity[PossibleContact.Key].Add(GeoIdx);
				}
			}
		}

	}

	TArray<TSet<int32>> MoveProximityToArray()
	{
		TArray<TSet<int32>> Proximity = MoveTemp(KnownProximity);
		for (int32 GeoIdx = 0; GeoIdx < Proximity.Num(); ++GeoIdx)
		{
			for (const TPair<int32, bool>& PossibleContact : GeoInfo[GeoIdx].CandidateContacts)
			{
				if (PossibleContact.Value)
				{
					Proximity[GeoIdx].Add(PossibleContact.Key);
					Proximity[PossibleContact.Key].Add(GeoIdx);
				}
			}
		}
		return Proximity;
	}

	bool AreFacesCoPlanar(const FGeometryCollection* Collection, int32 Idx0, int32 Idx1, float ProximityTolerance) const
	{
		// Assumes that faces have already been determined to be parallel.

		const TManagedArray<FIntVector>& Indices = Collection->Indices;

		FVector3f SamplePoint = TransformedVertices[Indices[Idx0].X];
		FVector3f PlaneOrigin = TransformedVertices[Indices[Idx1].X];
		FVector3f PlaneNormal = SurfaceNormals[Idx1];

		return FMath::Abs(FVector3f::DotProduct((SamplePoint - PlaneOrigin), PlaneNormal)) < ProximityTolerance;
	}

	bool DoFacesOverlap(const FGeometryCollection* Collection, int32 Idx0, int32 Idx1) const
	{
		// Assumes that faces have already been determined to be coplanar

		const TManagedArray<FIntVector>& Indices = Collection->Indices;

		// Project the first triangle into its normal plane
		FVector3f Basis0 = TransformedVertices[Indices[Idx0].Y] - TransformedVertices[Indices[Idx0].X];
		Basis0.Normalize();
		FVector3f Basis1 = FVector3f::CrossProduct(SurfaceNormals[Idx0], Basis0);
		Basis1.Normalize();

		FVector3f Origin = TransformedVertices[Indices[Idx0].X];

		TStaticArray<FVector2f, 3> T0;
		// T0[0] is the origin of the system
		T0[0] = FVector2f(0.f, 0.f);

		T0[1] = FVector2f(FVector3f::DotProduct(TransformedVertices[Indices[Idx0].Y] - Origin, Basis0), FVector3f::DotProduct(TransformedVertices[Indices[Idx0].Y] - Origin, Basis1));
		T0[2] = FVector2f(FVector3f::DotProduct(TransformedVertices[Indices[Idx0].Z] - Origin, Basis0), FVector3f::DotProduct(TransformedVertices[Indices[Idx0].Z] - Origin, Basis1));

		// Project the second triangle into these coordinates. We reverse the winding order to flip the normal.
		FVector3f Point0 = TransformedVertices[Indices[Idx1].Z] - Origin;
		FVector3f Point1 = TransformedVertices[Indices[Idx1].Y] - Origin;
		FVector3f Point2 = TransformedVertices[Indices[Idx1].X] - Origin;
		TStaticArray<FVector2f, 3> T1;
		T1[0] = FVector2f(FVector3f::DotProduct(Point0, Basis0), FVector3f::DotProduct(Point0, Basis1));
		T1[1] = FVector2f(FVector3f::DotProduct(Point1, Basis0), FVector3f::DotProduct(Point1, Basis1));
		T1[2] = FVector2f(FVector3f::DotProduct(Point2, Basis0), FVector3f::DotProduct(Point2, Basis1));

		// Note: If this was the only proximity test, we would also need to check for identical triangles here,
		// but we don't need to in practice because we cover that case w/ the proximity-from-vertices that we compute first
		return TrianglesIntersect(T0, T1);
	}

	void InitCandidateContacts(const FGeometryCollection* Collection, float ProximityTolerance)
	{
		UE::Geometry::FSparseDynamicOctree3 GeoOctree;
		GeoOctree.RootDimension = OverallBounds.GetExtent().GetAbsMax();
		FVector3d Center = (FVector3d)OverallBounds.GetCenter();
		TArray<int32> GeoIndices;
		for (int32 GeoIdx = 0; GeoIdx < GeoInfo.Num(); ++GeoIdx)
		{
			UE::Math::TBox<float> ExpandedBounds = GeoInfo[GeoIdx].Bounds.ExpandBy(ProximityTolerance);

			// Center the boxes to work better with the sparse dynamic octree
			UE::Geometry::FAxisAlignedBox3d CenteredBox3d((FVector3d)GeoInfo[GeoIdx].Bounds.Min, (FVector3d)GeoInfo[GeoIdx].Bounds.Max);
			CenteredBox3d.Min -= Center;
			CenteredBox3d.Max -= Center;
			if (GeoIdx > 0)
			{
				GeoIndices.Reset();
				UE::Geometry::FAxisAlignedBox3d CenteredBox3dExpanded = CenteredBox3d;
				CenteredBox3dExpanded.Expand(ProximityTolerance);
				GeoOctree.RangeQuery(CenteredBox3dExpanded, GeoIndices);
				for (int32 CandidateIdx : GeoIndices)
				{
					if (!KnownProximity[CandidateIdx].Contains(GeoIdx) &&
						// FSparseDynamicOctree3 doesn't doesn't filter for actual bounding box overlap, so we need to do so here
						ExpandedBounds.Intersect(GeoInfo[CandidateIdx].Bounds))
					{
						// Note: Only add the lower idx -> higher idx mapping
						GeoInfo[CandidateIdx].CandidateContacts.Add(GeoIdx, false);
					}
				}
			}
			GeoOctree.InsertObject(GeoIdx, CenteredBox3d);
		}
	}

	void InitGeoNormalBins(const FGeometryCollection* Collection)
	{
		FBinNormals Binner;
		ParallelFor(GeoInfo.Num(), [this, &Collection, &Binner](int32 GeoIdx)
			{
				GeoInfo[GeoIdx].InitBins(Collection, Binner, GeoIdx, SurfaceNormals);
			});
	}

	void InitProximityFromVertices(const FGeometryCollection* Collection, float ProximityTolerance)
	{
		UE::Geometry::TPointHashGrid3f<int32> VertHash(ProximityTolerance * 3, -1);
		TArray<int32> NearPts;
		int32 NumGeo = Collection->NumElements(FGeometryCollection::GeometryGroup);
		KnownProximity.SetNum(NumGeo);
		GeoInfo.SetNum(NumGeo);
		for (int32 GeoIdx = 0; GeoIdx < NumGeo; ++GeoIdx)
		{
			GeoInfo[GeoIdx].Bounds = UE::Math::TBox<float>(EForceInit::ForceInit);
			int32 VertStart = Collection->VertexStart[GeoIdx];
			int32 VertEnd = VertStart + Collection->VertexCount[GeoIdx];
			int32 TransformIdx = Collection->TransformIndex[GeoIdx];
			if (!Collection->IsRigid(TransformIdx))
			{
				continue;
			}
			for (int32 VertIdx = VertStart; VertIdx < VertEnd; VertIdx++)
			{
				const FVector3f& Vertex = TransformedVertices[VertIdx];
				GeoInfo[GeoIdx].Bounds += Vertex;
				NearPts.Reset();
				VertHash.FindPointsInBall(Vertex, ProximityTolerance, [&Vertex, this](const int32& Other) -> float
					{
						return FVector3f::DistSquared(TransformedVertices[Other], Vertex);
					}, NearPts);
				for (int32 NearPtIdx : NearPts)
				{
					int32 NearTransformIdx = Collection->BoneMap[NearPtIdx];
					if (NearTransformIdx != TransformIdx)
					{
						int32 NearGeoIdx = Collection->TransformToGeometryIndex[NearTransformIdx];
						KnownProximity[NearGeoIdx].Add(GeoIdx);
						KnownProximity[GeoIdx].Add(NearGeoIdx);
					}
				}
				VertHash.InsertPointUnsafe(VertIdx, Vertex);
			}
			OverallBounds += GeoInfo[GeoIdx].Bounds;
		}
	}

	void TransformVertices(const FGeometryCollection* Collection)
	{
		TransformedVertices.SetNum(Collection->NumElements(FGeometryCollection::VerticesGroup));

		TArray<FTransform> GlobalTransformArray;
		GeometryCollectionAlgo::GlobalMatrices(Collection->Transform, Collection->Parent, GlobalTransformArray);

		ParallelFor(Collection->NumElements(FGeometryCollection::VerticesGroup), [this, &Collection, &GlobalTransformArray](int32 VertIdx)
			{
				const TManagedArray<int32>& BoneMap = Collection->BoneMap;
				const TManagedArray<FVector3f>& Vertex = Collection->Vertex;

				const FTransform& GlobalTransform = GlobalTransformArray[BoneMap[VertIdx]];
				TransformedVertices[VertIdx] = (FVector3f)GlobalTransform.TransformPosition(FVector(Vertex[VertIdx]));
			});

	}

	void GenerateSurfaceNormals(const FGeometryCollection* Collection)
	{
		const int32 NumFaces = Collection->NumElements(FGeometryCollection::FacesGroup);
		// Generate surface normal for each face
		SurfaceNormals.SetNum(NumFaces);
		ParallelFor(NumFaces, [this, &Collection](int32 FaceIdx)
			{
				const TManagedArray<FIntVector>& Indices = Collection->Indices;

				FVector3f Edge0 = (TransformedVertices[Indices[FaceIdx].X] - TransformedVertices[Indices[FaceIdx].Y]);
				FVector3f Edge1 = (TransformedVertices[Indices[FaceIdx].Z] - TransformedVertices[Indices[FaceIdx].Y]);
				SurfaceNormals[FaceIdx] = FVector3f::CrossProduct(Edge0, Edge1);
				SurfaceNormals[FaceIdx].Normalize();
			});
	}

};

void BuildProximityFromConvexHulls(FGeometryCollection* Collection, const UE::GeometryCollectionConvexUtility::FConvexHulls& HullData, double DistanceThreshold)
{
	double MaxHullDim = DistanceThreshold;
	Chaos::FAABB3 OverallBounds;
	for (const Chaos::FConvexPtr& Hull : HullData.Hulls)
	{
		const Chaos::FAABB3 HullBounds = Hull->BoundingBox();
		OverallBounds.GrowToInclude(HullBounds);
		MaxHullDim = FMath::Max(MaxHullDim, HullBounds.Extents().GetMax() + DistanceThreshold);
	}

	if (!Collection->HasAttribute("Proximity", FGeometryCollection::GeometryGroup))
	{
		const FManagedArrayCollection::FConstructionParameters GeometryDependency(FGeometryCollection::GeometryGroup);
		Collection->AddAttribute<TSet<int32>>("Proximity", FGeometryCollection::GeometryGroup, GeometryDependency);
	}

	TManagedArray<TSet<int32>>& Proximity = Collection->ModifyAttribute<TSet<int32>>("Proximity", FGeometryCollection::GeometryGroup);


	UE::Geometry::FSparseDynamicOctree3 GeoOctree;
	GeoOctree.RootDimension = FMath::Min(MaxHullDim * 2, OverallBounds.Extents().GetMax() + DistanceThreshold);
	FVector3d Center = (FVector3d)OverallBounds.GetCenter();
	int32 NumGeometry = Collection->NumElements(FGeometryCollection::GeometryGroup);
	TArray<int32> HullToGeoIdx;
	HullToGeoIdx.Init(-1, HullData.Hulls.Num());
	for (int32 GeoIdx = 0; GeoIdx < NumGeometry; ++GeoIdx)
	{
		int32 TransformIdx = Collection->TransformIndex[GeoIdx];
		for (int32 HullIdx : HullData.TransformToHullsIndices[TransformIdx])
		{
			checkSlow(HullToGeoIdx[HullIdx] == -1); // these local-space hulls should not be mapped to multiple geometry pieces
			HullToGeoIdx[HullIdx] = GeoIdx;
		}
	}

	TArray<int32> HullIndices;
	for (int32 GeoIdx = 0; GeoIdx < NumGeometry; ++GeoIdx)
	{
		int32 TransformIdx = Collection->TransformIndex[GeoIdx];
		for (int32 HullIdx : HullData.TransformToHullsIndices[TransformIdx])
		{
			const Chaos::FConvex& Hull = *HullData.Hulls[HullIdx];
			Chaos::FAABB3 ChaosHullBounds = Hull.BoundingBox();
			ChaosHullBounds.Thicken(DistanceThreshold * .5);
			UE::Geometry::FAxisAlignedBox3d GeoBounds((FVector3d)ChaosHullBounds.Min(), (FVector3d)ChaosHullBounds.Max());
			GeoOctree.RangeQuery(GeoBounds, HullIndices);
			for (int32 CandidateHullIdx : HullIndices)
			{
				const Chaos::FConvex& CandidateHull = *HullData.Hulls[CandidateHullIdx];
				const Chaos::FAABB3 CandidateHullBounds = CandidateHull.BoundingBox();
				int32 OtherGeoIdx = HullToGeoIdx[CandidateHullIdx];

				if (GeoIdx != OtherGeoIdx && ChaosHullBounds.Intersects(CandidateHullBounds))
				{
					// TODO: Note the thickness parameter for GJKIntersection does not work as documented
					// But if that is fixed, we can replace the below GJKDistance call with this:
					//	const VectorRegister4Float InitialDirSimd = MakeVectorRegisterFloat(1.f, 0.f, 0.f, 0.f);
					//	if (GJKIntersectionSameSpaceSimd(Hull, CandidateHull, DistanceThreshold, InitialDirSimd))

					Chaos::FReal Distance;
					Chaos::TVec3<Chaos::FReal> NearestA, NearestB, Normal; // All unused
					Chaos::EGJKDistanceResult Result = Chaos::GJKDistance<Chaos::FReal>(
						Chaos::TGJKShape(Hull),
						Chaos::TGJKShape(CandidateHull),
						Chaos::GJKDistanceInitialVFromDirection(Hull, CandidateHull, CandidateHull.GetCenterOfMass() - Hull.GetCenterOfMass()),
						Distance, NearestA, NearestB, Normal);
					if (Result == Chaos::EGJKDistanceResult::Contact || Result == Chaos::EGJKDistanceResult::DeepContact
						|| (Result == Chaos::EGJKDistanceResult::Separated && Distance <= DistanceThreshold))
					{
						Proximity[GeoIdx].Add(OtherGeoIdx);
						Proximity[OtherGeoIdx].Add(GeoIdx);
					}
				}
			}
		}
		// add all the hulls for a given geometry after intersection-testing them with the hulls already in the octree
		for (int32 HullIdx : HullData.TransformToHullsIndices[TransformIdx])
		{
			const Chaos::FConvex& Hull = *HullData.Hulls[HullIdx];
			Chaos::FAABB3 ChaosHullBounds = Hull.BoundingBox();
			ChaosHullBounds.Thicken(DistanceThreshold * .5);
			UE::Geometry::FAxisAlignedBox3d GeoBounds((FVector3d)ChaosHullBounds.Min(), (FVector3d)ChaosHullBounds.Max());
			GeoOctree.InsertObject(HullIdx, GeoBounds);
		}
	}
}

}} // namespace UE::GeometryCollectionInternal

FGeometryCollectionProximityUtility::FGeometryCollectionProximityUtility(FGeometryCollection* InCollection)
	: Collection(InCollection)
{
	check(Collection);
}

void FGeometryCollectionProximityUtility::RequireProximity(UE::GeometryCollectionConvexUtility::FConvexHulls* OptionalComputedHulls)
{
	bool bHasProximity = Collection->HasAttribute("Proximity", FGeometryCollection::GeometryGroup);
	if (!bHasProximity)
	{
		UpdateProximity(OptionalComputedHulls);
	}
}

void FGeometryCollectionProximityUtility::InvalidateProximity()
{
	bool bHasProximity = Collection->HasAttribute("Proximity", FGeometryCollection::GeometryGroup);
	if (bHasProximity)
	{
		Collection->RemoveAttribute("Proximity", FGeometryCollection::GeometryGroup);
	}
}

void FGeometryCollectionProximityUtility::ClearConnectionGraph()
{
	GeometryCollection::Facades::FCollectionConnectionGraphFacade ConnectionsFacade(*Collection);
	ConnectionsFacade.ClearAttributes();
}

void FGeometryCollectionProximityUtility::EnumerateNeighbors(const Chaos::Facades::FCollectionHierarchyFacade& Hierarchy, 
	int32 TransformIdx, TFunctionRef<void(int32)> NeighborFunc, bool bIncludeNeighborsInParentLevels, bool bFilterDuplicates)
{
	if (!ensureMsgf(Collection->HasAttribute("Proximity", FGeometryCollection::GeometryGroup),
		TEXT("Attempted to enumerate geometry collection neighbors, but proximity graph was not available")))
	{
		return;
	}
	if (!Hierarchy.IsValid() || !Hierarchy.HasLevelAttribute())
	{
		ensureMsgf(false, TEXT("Must have a valid hierarchy with a level attribute to enumerate geometry collection neighbors"));
	}

	TSet<int32> AlreadyProcessed;

	int32 Level = Hierarchy.GetInitialLevel(TransformIdx);

	const TManagedArray<TSet<int32>>& Proximity = Collection->GetAttribute<TSet<int32>>("Proximity", FGeometryCollection::GeometryGroup);
	const TManagedArray<int32>& GeoToTransformIndex = Collection->GetAttribute<int32>("TransformIndex", FGeometryCollection::GeometryGroup);
	const TManagedArray<int32>& TransformToGeometryIndex = Collection->GetAttribute<int32>("TransformToGeometryIndex", FGeometryCollection::TransformGroup);
	const TManagedArray<int32>& SimulationType = Collection->GetAttribute<int32>("SimulationType", FGeometryCollection::TransformGroup);
	Hierarchy.EnumerateChildTransforms(TransformIdx, 
		[&SimulationType](int32 TransformIndex)
		{
			return SimulationType[TransformIndex] == (int32)FGeometryCollection::ESimulationTypes::FST_Rigid;
		}, 
		[&Hierarchy, TransformIdx, &NeighborFunc, bIncludeNeighborsInParentLevels, bFilterDuplicates, &AlreadyProcessed, Level, &Proximity, &GeoToTransformIndex, &TransformToGeometryIndex, &SimulationType]
		(int32 ChildTransformIndex)
		{
			int32 GeoIdx = TransformToGeometryIndex[ChildTransformIndex];
			for (int32 ProxGeoIndex : Proximity[GeoIdx])
			{
				int32 ProxTransformIndex = GeoToTransformIndex[ProxGeoIndex];
				int32 ProxLevel = Hierarchy.GetInitialLevel(ProxTransformIndex);
				if (bIncludeNeighborsInParentLevels && ProxLevel < Level)
				{
					NeighborFunc(ProxTransformIndex);
				}
				while (ProxLevel > Level && ProxTransformIndex != INDEX_NONE)
				{
					ProxTransformIndex = Hierarchy.GetParent(ProxTransformIndex);
					ProxLevel--;
				}
				if (ProxTransformIndex != TransformIdx && ProxTransformIndex != INDEX_NONE)
				{
					if (bFilterDuplicates)
					{
						if (AlreadyProcessed.Contains(ProxTransformIndex))
						{
							continue;
						}
						AlreadyProcessed.Add(ProxTransformIndex);
					}
					NeighborFunc(ProxTransformIndex);
				}
			}
		});
}

void FGeometryCollectionProximityUtility::CopyProximityToConnectionGraph(const TArray<FGeometryContactEdge>* ContactEdges)
{
	if (!Collection->HasAttribute("Proximity", FGeometryCollection::GeometryGroup))
	{
		ClearConnectionGraph();
		return;
	}

	GeometryCollection::Facades::FCollectionConnectionGraphFacade ConnectionsFacade(*Collection);
	ConnectionsFacade.DefineSchema();
	ConnectionsFacade.ResetConnections();

	Chaos::Facades::FCollectionHierarchyFacade HierarchyFacade(*Collection);
	HierarchyFacade.GenerateLevelAttribute();

	TMap<UE::Geometry::FIndex2i, float> ContactSum;
	// Helper to accumulate contact from a geometry connection to the appropriate transform connection
	auto ProcessGeometryContact = [this, &ConnectionsFacade, &HierarchyFacade, &ContactSum](int32 GeometryA, int32 GeometryB, bool bAccumulateContactAreas = false, float Contact = 1.f)
	{
		int32 Bones[2]{ Collection->TransformIndex[GeometryA], Collection->TransformIndex[GeometryB] };
		int32 Levels[2]{ HierarchyFacade.GetInitialLevel(Bones[0]), HierarchyFacade.GetInitialLevel(Bones[1]) };
		int32 Parents[2]{ HierarchyFacade.GetParent(Bones[0]), HierarchyFacade.GetParent(Bones[1]) };
		while (Parents[0] != INDEX_NONE && Parents[1] != INDEX_NONE)
		{
			if (Bones[0] == Bones[1]) // stop if both ends are the same bone
			{
				break;
			}
			if (Parents[0] == Parents[1])
			{
				UE::Geometry::FIndex2i ConnectBones(Bones[0], Bones[1]);
				if (Bones[0] > Bones[1])
				{
					Swap(ConnectBones.A, ConnectBones.B);
				}
				float* FoundContact = ContactSum.Find(ConnectBones);
				if (!FoundContact)
				{
					ContactSum.Add(ConnectBones, Contact);
				}
				else if (bAccumulateContactAreas)
				{
					*FoundContact += Contact;
				}
				break;
			}

			bool RaiseLevels[2]{ Levels[0] >= Levels[1], Levels[1] >= Levels[0] };
			for (int32 Idx = 0; Idx < 2; ++Idx)
			{
				if (RaiseLevels[Idx])
				{
					Bones[Idx] = Parents[Idx];
					Parents[Idx] = HierarchyFacade.GetParent(Bones[Idx]);
					Levels[Idx]--;
				}
			}
		}
	};

	FGeometryCollectionProximityPropertiesInterface::FProximityProperties Properties = Collection->GetProximityProperties();
	if (Properties.ContactAreaMethod == EConnectionContactMethod::ConvexHullContactArea)
	{
		ConnectionsFacade.EnableContactAreas(true);
		TArray<FGeometryContactEdge> LocalContactEdges;
		const TArray<FGeometryContactEdge>* UseContactEdges = ContactEdges;
		if (!UseContactEdges)
		{
			FGeometryCollectionConvexPropertiesInterface::FConvexCreationProperties ConvexProperties = Collection->GetConvexProperties();
			TArray<FTransform> GlobalTransformArray;
			GeometryCollectionAlgo::GlobalMatrices(Collection->Transform, Collection->Parent, GlobalTransformArray);
			UE::GeometryCollectionConvexUtility::FConvexHulls ComputedHulls = FGeometryCollectionConvexUtility::ComputeLeafHulls(Collection, GlobalTransformArray, ConvexProperties.SimplificationThreshold, 0.0f /* Never shrink hulls for proximity detection */);

			LocalContactEdges = ComputeConvexGeometryContactFromProximity(Collection, Properties.DistanceThreshold, ComputedHulls);

			UseContactEdges = &LocalContactEdges;
		}
		for (const FGeometryContactEdge& Edge : *UseContactEdges)
		{
			ProcessGeometryContact(Edge.GeometryIndices[0], Edge.GeometryIndices[1], true, Edge.ContactArea);
		}
	}
	else // No contact areas
	{
		ConnectionsFacade.EnableContactAreas(false);
		const TManagedArray<TSet<int32>>& Proximity = Collection->GetAttribute<TSet<int32>>("Proximity", FGeometryCollection::GeometryGroup);
		for (int32 GeoIdx = 0; GeoIdx < Proximity.Num(); ++GeoIdx)
		{
			for (int32 NbrIdx : Proximity[GeoIdx])
			{
				ProcessGeometryContact(GeoIdx, NbrIdx, false);
			}
		}
	}

	// Copy out final contacts
	for (TPair<UE::Geometry::FIndex2i, float> Contact : ContactSum)
	{
		if (ConnectionsFacade.HasContactAreas())
		{
			ConnectionsFacade.ConnectWithContact(Contact.Key.A, Contact.Key.B, Contact.Value);
		}
		else
		{
			ConnectionsFacade.Connect(Contact.Key.A, Contact.Key.B);
		}
	}
}

TArray<TSet<int32>> FGeometryCollectionProximityUtility::ComputePreciseProximity(const FGeometryCollection& Collection)
{
	using namespace UE::GeometryCollectionInternal;
	FGeometryCollectionProximitySpatial Spatial(&Collection, PreciseProximityThreshold);
	return Spatial.MoveProximityToArray();
}

void FGeometryCollectionProximityUtility::UpdateProximity(UE::GeometryCollectionConvexUtility::FConvexHulls* OptionalComputedHulls)
{
	using namespace UE::GeometryCollectionInternal;

	FGeometryCollectionProximityPropertiesInterface::FProximityProperties Properties = Collection->GetProximityProperties();

	bool bWantConvexContactEdges = 
		(Properties.RequireContactAmount > 0.0f &&
		(Properties.ContactMethod == EProximityContactMethod::ConvexHullSharpContact || Properties.ContactMethod == EProximityContactMethod::ConvexHullAreaContact))
		||
		(Properties.ContactAreaMethod == EConnectionContactMethod::ConvexHullContactArea);
	bool bWantLocalHulls = Properties.Method == EProximityMethod::ConvexHull || bWantConvexContactEdges;

	FGeometryCollectionConvexPropertiesInterface::FConvexCreationProperties ConvexProperties = Collection->GetConvexProperties();

	UE::GeometryCollectionConvexUtility::FConvexHulls LocalComputedHulls;
	UE::GeometryCollectionConvexUtility::FConvexHulls* UseComputedHulls = OptionalComputedHulls;
	if (bWantLocalHulls && (!UseComputedHulls || UseComputedHulls->OverlapRemovalShrinkPercent > 0)) // If we don't have precomputed hulls or if they're shrunk, compute new hulls to use for proximity detection
	{
		TArray<FTransform> GlobalTransformArray;
		GeometryCollectionAlgo::GlobalMatrices(Collection->Transform, Collection->Parent, GlobalTransformArray);
		LocalComputedHulls = FGeometryCollectionConvexUtility::ComputeLeafHulls(Collection, GlobalTransformArray, ConvexProperties.SimplificationThreshold, 0.0f /* Never shrink hulls for proximity detection */);
		UseComputedHulls = &LocalComputedHulls;
	}

	if (Properties.Method == EProximityMethod::ConvexHull)
	{
		UE::GeometryCollectionInternal::BuildProximityFromConvexHulls(Collection, *UseComputedHulls, Properties.DistanceThreshold);
	}
	else
	{
		FGeometryCollectionProximitySpatial Spatial(Collection, PreciseProximityThreshold);
		Spatial.MoveProximityToCollection(Collection);
	}

	TArray<FGeometryContactEdge> ContactEdges;
	if (bWantConvexContactEdges)
	{
		ContactEdges = ComputeConvexGeometryContactFromProximity(Collection, Properties.DistanceThreshold, *UseComputedHulls);
	}

	if (Properties.RequireContactAmount > 0.0f)
	{
		if (Properties.ContactMethod == EProximityContactMethod::MinOverlapInProjectionToMajorAxes)
		{
			TManagedArray<TSet<int32>>& Proximity = Collection->ModifyAttribute<TSet<int32>>("Proximity", FGeometryCollection::GeometryGroup);
			int32 NumGeometry = Collection->NumElements(FGeometryCollection::GeometryGroup);

			TArray<FBox> GeometryBounds; // Geometry bounding boxes in a shared space
			GeometryBounds.Init(FBox(EForceInit::ForceInit), NumGeometry);

			TArray<FTransform> GlobalTransformArray;
			GeometryCollectionAlgo::GlobalMatrices(Collection->Transform, Collection->Parent, GlobalTransformArray);

			ParallelFor(NumGeometry, [&](int32 GeometryIdx)
			{
				FBox& Box = GeometryBounds[GeometryIdx];
				const int32 Start = Collection->VertexStart[GeometryIdx];
				const int32 End = Start + Collection->VertexCount[GeometryIdx];
				FTransform GeometryTransform = GlobalTransformArray[Collection->TransformIndex[GeometryIdx]];
				for (int32 VertIdx = Start; VertIdx < End; ++VertIdx)
				{
					Box += GeometryTransform.TransformPosition((FVector)Collection->Vertex[VertIdx]);
				}
			});

			TArray<int32> ToRemove;
			auto ProjectBox = [](const FBox& Box, int32 Axis) -> FBox2D
			{
				int32 X = (Axis + 1) % 3;
				int32 Y = (Axis + 2) % 3;
				return FBox2D(FVector2D(Box.Min[X], Box.Min[Y]), FVector2D(Box.Max[X], Box.Max[Y]));
			};
			for (int32 GeometryIdx = 0; GeometryIdx < NumGeometry; ++GeometryIdx)
			{
				FBox Box = GeometryBounds[GeometryIdx];
				ToRemove.Reset();
				for (int32 ConnectedGeoIdx : Proximity[GeometryIdx])
				{
					FBox OtherBox = GeometryBounds[ConnectedGeoIdx];
					bool bOverlapAnyAxis = false;
					for (int ProjAxis = 0; ProjAxis < 3; ++ProjAxis)
					{
						FBox2D ProjA = ProjectBox(Box, ProjAxis);
						FBox2D ProjB = ProjectBox(OtherBox, ProjAxis);
						FBox2D Overlap = ProjA.Overlap(ProjB);
						if (Overlap.bIsValid)
						{
							float MinBoundsAxis = (float)FMath::Min(ProjA.GetSize().GetMin(), ProjB.GetSize().GetMin());
							float MinAxisOverlap = (float)Overlap.GetSize().GetMin();
							// Overlap is accepted if it is greater than the threshold amount OR greater than half the maximum possible amount (to avoid 100% filtering small pieces)
							if (MinAxisOverlap > FMath::Min(MinBoundsAxis*.5f, Properties.RequireContactAmount))
							{
								bOverlapAnyAxis = true;
								break;
							}
						}
					}
					if (!bOverlapAnyAxis)
					{
						ToRemove.Add(ConnectedGeoIdx);
					}
				}
				for (int32 Nbr : ToRemove)
				{
					Proximity[GeometryIdx].Remove(Nbr);
					Proximity[Nbr].Remove(GeometryIdx);
				}
			}
		}
		else if (Properties.ContactMethod == EProximityContactMethod::ConvexHullSharpContact)
		{
			TManagedArray<TSet<int32>>& Proximity = Collection->ModifyAttribute<TSet<int32>>("Proximity", FGeometryCollection::GeometryGroup);
			for (const FGeometryContactEdge& Edge : ContactEdges)
			{
				if (Edge.SharpContactWidth < Properties.RequireContactAmount && Edge.SharpContactWidth < Edge.MaxSharpContact * .5)
				{
					Proximity[Edge.GeometryIndices[0]].Remove(Edge.GeometryIndices[1]);
					Proximity[Edge.GeometryIndices[1]].Remove(Edge.GeometryIndices[0]);
				}
			}
		}
		else if (Properties.ContactMethod == EProximityContactMethod::ConvexHullAreaContact)
		{
			TManagedArray<TSet<int32>>& Proximity = Collection->ModifyAttribute<TSet<int32>>("Proximity", FGeometryCollection::GeometryGroup);
			for (const FGeometryContactEdge& Edge : ContactEdges)
			{
				float RequireArea = Properties.RequireContactAmount * Properties.RequireContactAmount;
				if (Edge.ContactArea < RequireArea && Edge.ContactArea < Edge.MaxContactArea * .5)
				{
					Proximity[Edge.GeometryIndices[0]].Remove(Edge.GeometryIndices[1]);
					Proximity[Edge.GeometryIndices[1]].Remove(Edge.GeometryIndices[0]);
				}
			}
		}
	}

	if (Properties.bUseAsConnectionGraph)
	{
		CopyProximityToConnectionGraph(Properties.ContactAreaMethod == EConnectionContactMethod::ConvexHullContactArea  ? &ContactEdges : nullptr);
	}
	else
	{
		// TODO: verify that the connection graph was auto-generated from proximity, rather than being custom edited, before removal
		ClearConnectionGraph();
	}
}







TArray<FGeometryCollectionProximityUtility::FGeometryContactEdge>
FGeometryCollectionProximityUtility::ComputeConvexGeometryContactFromProximity(
	FGeometryCollection* Collection, float DistanceTolerance,
	UE::GeometryCollectionConvexUtility::FConvexHulls& LocalHulls)
{
	// We must already have proximity
	const TManagedArray<TSet<int32>>* Proximity = Collection->FindAttribute<TSet<int32>>("Proximity", FGeometryCollection::GeometryGroup);
	if (!ensureMsgf(Proximity, TEXT("ComputeGeometryContactFromProximity should only be called when a Proximity attribute has already been added")))
	{
		return TArray<FGeometryCollectionProximityUtility::FGeometryContactEdge>();
	}
	
	TArray<FGeometryCollectionProximityUtility::FGeometryContactEdge> ContactEdges;
	for (int32 GeoIdx = 0; GeoIdx < Proximity->Num(); ++GeoIdx)
	{
		int32 TransformIdx = Collection->TransformIndex[GeoIdx];
		const TSet<int32>& GeoHulls = LocalHulls.TransformToHullsIndices[TransformIdx];
		for (int32 NbrGeoIdx : (*Proximity)[GeoIdx])
		{
			// all connections should be symmetric, and no connections should be to self, so only compute lower idx->higher idx
			if (GeoIdx >= NbrGeoIdx)
			{
				continue;
			}
			int32 NbrTransformIdx = Collection->TransformIndex[NbrGeoIdx];
			const TSet<int32>& NbrGeoHulls = LocalHulls.TransformToHullsIndices[NbrTransformIdx];
			float OverlapAreas = 0, OverlapMaxAreas = 0, OverlapSharpContact = 0, OverlapMaxSharpContact = 0;
			for (int32 GeoHullIdx : GeoHulls)
			{
				for (int32 NbrHullIdx : NbrGeoHulls)
				{
					float Area, MaxArea, SharpContact, MaxSharpContact;
					UE::GeometryCollectionConvexUtility::HullIntersectionStats(LocalHulls.Hulls[GeoHullIdx].GetReference(), LocalHulls.Hulls[NbrHullIdx].GetReference(), DistanceTolerance, Area, MaxArea, SharpContact, MaxSharpContact);
					// Note: The sharp contact concept would be most accurately implemented by combining the intersection/overlaps of all involved hulls
					// but taking the max values should be a reasonable conservative approximation in most cases
					OverlapSharpContact = FMath::Max(OverlapSharpContact, SharpContact);
					OverlapMaxSharpContact = FMath::Max(OverlapMaxSharpContact, MaxSharpContact);
					OverlapMaxAreas += MaxArea;
					OverlapAreas += Area;
				}
			}
			// to approximate contact surface area, use half of surface area of convex hull intersection
			ContactEdges.Emplace(GeoIdx, NbrGeoIdx, OverlapAreas * .5f, OverlapMaxAreas * .5f, OverlapSharpContact, OverlapMaxSharpContact);
		}
	}

	return ContactEdges;
}
