// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionSection.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/GeometryCollection.h"


TArray<FGeometryCollectionSection>
FGeometryCollectionSection::BuildMeshSections(const FManagedArrayCollection& InCollection, const TArray<FIntVector>& InputIndices, const TArray<int32>& BaseMeshOriginalIndicesIndex, TArray<FIntVector>& RetIndices)
{
	TArray<FGeometryCollectionSection> RetSections;
	TArray<FGeometryCollectionSection> TmpSections;
	if (InCollection.FindAttributeTyped<int32>("MaterialID", FGeometryCollection::FacesGroup))
	{
		const TManagedArray<int32>& MaterialID = InCollection.GetAttribute<int32>("MaterialID", FGeometryCollection::FacesGroup);

		// count the number of triangles for each material section, adding a new section if the material ID is higher than the current number of sections
		for (int FaceElement = 0; FaceElement < InputIndices.Num(); ++FaceElement)
		{
			int32 Section = MaterialID[BaseMeshOriginalIndicesIndex[FaceElement]];

			while (Section + 1 > TmpSections.Num())
			{
				// add a new material section
				int32 Element = TmpSections.AddZeroed();

				TmpSections[Element].MaterialID = Element;
				TmpSections[Element].FirstIndex = -1;
				TmpSections[Element].NumTriangles = 0;
				TmpSections[Element].MinVertexIndex = 0;
				TmpSections[Element].MaxVertexIndex = 0;
			}

			TmpSections[Section].NumTriangles++;
		}

		// fixup the section FirstIndex and MaxVertexIndex
		for (int SectionElement = 0; SectionElement < TmpSections.Num(); SectionElement++)
		{
			if (SectionElement == 0)
			{
				TmpSections[SectionElement].FirstIndex = 0;
			}
			else
			{
				// Each subsequent section has an index that starts after the last one
				// note the NumTriangles*3 - this is because indices are sent to the renderer in a flat array
				TmpSections[SectionElement].FirstIndex = TmpSections[SectionElement - 1].FirstIndex + TmpSections[SectionElement - 1].NumTriangles * 3;
			}

			TmpSections[SectionElement].MaxVertexIndex = InCollection.NumElements(FGeometryCollection::VerticesGroup) - 1;
		}

		// remap indices so the materials appear to be grouped
		RetIndices.AddUninitialized(InputIndices.Num());

		// since we know the number of trinagle per section we can distribute the faces in RetIndices
		// this avoid nested loop that result in N * M operations (N=sections M=faces) and we can process it in (N + M) operations instead 
		TArray<int32> OffsetPerSection;
		OffsetPerSection.AddUninitialized(TmpSections.Num());
		int32 SectionOffset = 0;
		for (int Section = 0; Section < TmpSections.Num(); Section++)
		{
			OffsetPerSection[Section] = SectionOffset;
			SectionOffset += TmpSections[Section].NumTriangles;
		}

		for (int FaceElement = 0; FaceElement < InputIndices.Num(); FaceElement++)
		{
			const int32 SectionID = (MaterialID)[BaseMeshOriginalIndicesIndex[FaceElement]];
			int32& SectionOffsetRef = OffsetPerSection[SectionID];
			RetIndices[SectionOffsetRef++] = InputIndices[FaceElement];
		}

		// if a material group no longer has any triangles in it then add material section for removal
		RetSections.Reserve(TmpSections.Num());
		for (int SectionElement = 0; SectionElement < TmpSections.Num(); SectionElement++)
		{
			if (TmpSections[SectionElement].NumTriangles > 0)
			{
				RetSections.Push(TmpSections[SectionElement]);
			}
		}
	}
	return MoveTemp(RetSections);
}