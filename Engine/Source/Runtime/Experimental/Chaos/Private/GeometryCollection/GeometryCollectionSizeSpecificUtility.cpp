// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GeometryCollection->cpp: FGeometryCollection methods.
=============================================================================*/

#include "GeometryCollection/GeometryCollectionSizeSpecificUtility.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"

#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"
#include "Chaos/Real.h"

DEFINE_LOG_CATEGORY_STATIC(FGeometryCollectionSizeSpecificUtilityLogging, Log, All);

namespace GeometryCollection
{
	namespace SizeSpecific
	{
		int32 FindIndexForVolume(const TArray<FSharedSimulationSizeSpecificData>& SizeSpecificData, Chaos::FReal Volume)
		{
			check(SizeSpecificData.Num());
			int32 UseIdx = 0;
			float PreSize = FLT_MAX;
			for (int32 Idx = SizeSpecificData.Num() - 1; Idx >= 0; --Idx)
			{
				ensureMsgf(PreSize >= SizeSpecificData[Idx].MaxSize, TEXT("SizeSpecificData is not sorted"));
				PreSize = SizeSpecificData[Idx].MaxSize;
				if (Volume < SizeSpecificData[Idx].MaxSize)
					UseIdx = Idx;
				else
					break;
			}
			return UseIdx;
		}

		int32 FindIndexForVolume(const TArray<FSharedSimulationSizeSpecificData>& SizeSpecificData, const FBox& Bounds)
		{
			return FindIndexForVolume(SizeSpecificData, Bounds.GetVolume());
		}

	}
}