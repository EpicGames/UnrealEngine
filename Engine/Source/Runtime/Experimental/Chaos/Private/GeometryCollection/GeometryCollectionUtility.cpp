// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GeometryCollection->cpp: FGeometryCollection methods.
=============================================================================*/

#include "GeometryCollection/GeometryCollectionUtility.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"

#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"
#include "Chaos/Real.h"

DEFINE_LOG_CATEGORY_STATIC(FGeometryCollectionUtilityLogging, Log, All);

namespace GeometryCollection
{
	TSharedPtr<FGeometryCollection> MakeCubeElement(const FTransform& center, FVector Scale, int NumberOfMaterials)
	{
		FGeometryCollection* RestCollection = new FGeometryCollection();

		int NumNewVertices = 8; // 8 vertices per cube
		int VerticesIndex = RestCollection->AddElements(NumNewVertices, FGeometryCollection::VerticesGroup);
		
		int NumNewIndices = 2 * 6; // two triangles per face
		int IndicesIndex = RestCollection->AddElements(NumNewIndices, FGeometryCollection::FacesGroup);
		
		int NumNewParticles = 1; // 1 particle for this geometry structure
		int ParticlesIndex = RestCollection->AddElements(NumNewParticles, FGeometryCollection::TransformGroup);

		TManagedArray<FVector3f>& Vertices = RestCollection->Vertex;
		TManagedArray<FVector3f>&  Normals = RestCollection->Normal;
		TManagedArray<FVector3f>&  TangentU = RestCollection->TangentU;
		TManagedArray<FVector3f>&  TangentV = RestCollection->TangentV;
		TManagedArray<FVector2f>& UV0 = *RestCollection->FindUVLayer(0);
		TManagedArray<FLinearColor>&  Colors = RestCollection->Color;
		TManagedArray<FIntVector>&  Indices = RestCollection->Indices;
		TManagedArray<bool>&  Visible = RestCollection->Visible;
		TManagedArray<int32>&  MaterialIndex = RestCollection->MaterialIndex;
		TManagedArray<int32>&  MaterialID = RestCollection->MaterialID;
		TManagedArray<bool>& Internal = RestCollection->Internal;
		TManagedArray<FTransform3f>&  Transform = RestCollection->Transform;
		TManagedArray<int32>& SimType = RestCollection->SimulationType;
		TManagedArray<int32>& BoneMap = RestCollection->BoneMap;

		// set the particle information
		Transform[0] = FTransform3f(center);
		Transform[0].NormalizeRotation();
		SimType[0] = FGeometryCollection::ESimulationTypes::FST_Rigid;

		// set the vertex information
		int32 Index = 0;
		FVector3f Scale3f = FVector3f(Scale);	// LWC_TODO: Precision loss - Not significant as everything it applies to is already float.
		Vertices[0] = FVector3f(-Scale3f.X / 2.f, -Scale3f.Y / 2.f, -Scale3f.Z / 2.f);
		Vertices[1] = FVector3f(+Scale3f.X / 2.f, -Scale3f.Y / 2.f, -Scale3f.Z / 2.f);
		Vertices[2] = FVector3f(-Scale3f.X / 2.f, +Scale3f.Y / 2.f, -Scale3f.Z / 2.f);
		Vertices[3] = FVector3f(+Scale3f.X / 2.f, +Scale3f.Y / 2.f, -Scale3f.Z / 2.f);
		Vertices[4] = FVector3f(-Scale3f.X / 2.f, -Scale3f.Y / 2.f, +Scale3f.Z / 2.f);
		Vertices[5] = FVector3f(+Scale3f.X / 2.f, -Scale3f.Y / 2.f, +Scale3f.Z / 2.f);
		Vertices[6] = FVector3f(-Scale3f.X / 2.f, +Scale3f.Y / 2.f, +Scale3f.Z / 2.f);
		Vertices[7] = FVector3f(+Scale3f.X / 2.f, +Scale3f.Y / 2.f, +Scale3f.Z / 2.f);

		Normals[0] = FVector3f(-1.f, -1.f, -1.f).GetSafeNormal();
		Normals[1] = FVector3f(1.f, -1.f, -1.f).GetSafeNormal();
		Normals[2] = FVector3f(-1.f, 1.f, -1.f).GetSafeNormal();
		Normals[3] = FVector3f(1.f, 1.f, -1.f).GetSafeNormal();
		Normals[4] = FVector3f(-1.f, -1.f, 1.f).GetSafeNormal();
		Normals[5] = FVector3f(1.f, -1.f, 1.f).GetSafeNormal();
		Normals[6] = FVector3f(-1.f, 1.f, 1.f).GetSafeNormal();
		Normals[7] = FVector3f(1.f, 1.f, 1.f).GetSafeNormal();

		UV0[0] = FVector2f(0, 0);
		UV0[1] = FVector2f(1, 0);
		UV0[2] = FVector2f(0, 1);
		UV0[3] = FVector2f(1, 1);
		UV0[4] = FVector2f(0, 0);
		UV0[5] = FVector2f(1, 0);
		UV0[6] = FVector2f(0, 1);
		UV0[7] = FVector2f(1, 1);

		Colors[0] = FLinearColor::White;
		Colors[1] = FLinearColor::White;
		Colors[2] = FLinearColor::White;
		Colors[3] = FLinearColor::White;
		Colors[4] = FLinearColor::White;
		Colors[5] = FLinearColor::White;
		Colors[6] = FLinearColor::White;
		Colors[7] = FLinearColor::White;

		BoneMap[0] = 0;
		BoneMap[1] = 0;
		BoneMap[2] = 0;
		BoneMap[3] = 0;
		BoneMap[4] = 0;
		BoneMap[5] = 0;
		BoneMap[6] = 0;
		BoneMap[7] = 0;


		// set the index information

		// Bottom: Y = -1
		Indices[0] = FIntVector(Index + 5,Index + 1,Index);
		Indices[1] = FIntVector(Index,Index + 4,Index + 5);
		// Top: Y = 1
		Indices[2] = FIntVector(Index + 2,Index + 3,Index + 7);
		Indices[3] = FIntVector(Index + 7,Index + 6,Index + 2);
		// Back: Z = -1
		Indices[4] = FIntVector(Index + 3,Index + 2,Index);
		Indices[5] = FIntVector(Index,Index + 1,Index + 3);
		// Front: Z = 1
		Indices[6] = FIntVector(Index + 4,Index + 6,Index + 7);
		Indices[7] = FIntVector(Index + 7,Index + 5,Index + 4);
		// Left: X = -1
		Indices[8] = FIntVector(Index, Index + 2,Index + 6);
		Indices[9] = FIntVector(Index + 6,Index + 4,Index);
		// Right: X = 1
		Indices[10] = FIntVector(Index + 7,Index + 3,Index + 1);
		Indices[11] = FIntVector(Index + 1,Index + 5,Index + 7);

		// distribute the number of materials equally between the 12 faces
		check(NumberOfMaterials <= 12 && (12 % NumberOfMaterials)==0); // preferably divisible into 12
		int NumberOfEachMaterial = 12 / NumberOfMaterials;
		for (int i = 0; i < 12; i++)
		{
			Visible[i] = true;
			Internal[i] = false;

			MaterialIndex[i] = i;
			MaterialID[i] = i / NumberOfEachMaterial;
		}

		for (int IndexIdx = 0; IndexIdx < 12; IndexIdx++)
		{
			FIntVector Tri = Indices[IndexIdx];
			for (int idx = 0; idx < 3; idx++)
			{
				const FVector3f Normal = Normals[Tri[idx]];
				const FVector3f Edge = (Vertices[Tri[(idx + 1) % 3]] - Vertices[Tri[idx]]);
				TangentU[Tri[idx]] = (Edge ^ Normal).GetSafeNormal();
				TangentV[Tri[idx]] = (Normal ^ TangentU[Tri[idx]]).GetSafeNormal();
			}
		}

		// GeometryGroup
		GeometryCollection::AddGeometryProperties(RestCollection);

		// add the material sections to simulate NumberOfMaterials on the object
		TManagedArray<FGeometryCollectionSection>&  Sections = RestCollection->Sections;


		// the first 6 indices are material 0
		int FirstElement = RestCollection->AddElements(NumberOfMaterials, FGeometryCollection::MaterialGroup);
		for (int Element = 0; Element < NumberOfMaterials; Element++)
		{
			Sections[Element].MaterialID = Element;
			Sections[Element].FirstIndex = (Element * NumberOfEachMaterial) * 3;
			Sections[Element].NumTriangles = NumberOfEachMaterial;
			Sections[Element].MinVertexIndex = 0;
			Sections[Element].MaxVertexIndex = Vertices.Num() - 1;
		}

		return TSharedPtr<FGeometryCollection>(RestCollection);
	}


	void SetupCubeGridExample(TSharedPtr<FGeometryCollection> RestCollectionIn)
	{
		check(RestCollectionIn.IsValid());

		float domain = 10.f;
		FVector Stack(domain);
		float numElements = powf(domain, 3);

		float Length = 50.f;
		float Seperation = .2f;
		float Expansion = 1.f + Seperation;

		FVector Stackf((float)Stack[0], (float)Stack[1], (float)Stack[2]);
		FVector MinCorner = -Length * Expansion / 2.f * Stackf;


		for (int32 i = 0; i < FMath::TruncToInt(Stack[0]); ++i)
		{
			for (int32 j = 0; j < FMath::TruncToInt(Stack[1]); ++j)
			{
				for (int32 k = 0; k < FMath::TruncToInt(Stack[2]); ++k)
				{
					FVector Delta(j % 2 == 1 ? Length / 2.f : 0.f, 0.f, j % 2 == 1 ? Length / 2.f : 0.f);
					FVector CenterOfMass = FVector(MinCorner[0] + Expansion * Length * static_cast<Chaos::FReal>(i) + Length * (Expansion / 2.f),
						MinCorner[0] + Expansion * Length * static_cast<Chaos::FReal>(j) + Length * (Expansion / 2.f),
						MinCorner[0] + Expansion * Length * static_cast<Chaos::FReal>(k) + Length * (Expansion / 2.f)) + Delta;
					TSharedPtr<FGeometryCollection> Element = MakeCubeElement(FTransform(CenterOfMass), FVector(Length) );
					RestCollectionIn->AppendGeometry(*Element);
				}
			}
		}
	}

	void SetupTwoClusteredCubesCollection(FGeometryCollection * Collection)
	{
		int32 ParentIndex = Collection->AddElements(1, FGeometryCollection::TransformGroup);
		int32 TransformIndex0 = Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0, 0, 0.)), FVector(9, 0, 0)), FVector(1.0)));
		int32 TransformIndex1 = Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0, 0, 0.)), FVector(-9, 0, 0)), FVector(1.0)));

		TManagedArray<int32>& VertexCount = Collection->VertexCount;
		TManagedArray<int32>& VertexStart = Collection->VertexStart;
		TManagedArray<FVector3f> & Vertex = Collection->Vertex;
		
		TArray<int32> ReverseMap;
		GeometryCollectionAlgo::BuildTransformGroupToGeometryGroupMap(*Collection, ReverseMap);

		for (int32 i = VertexStart[ReverseMap[TransformIndex0]]; i < VertexStart[ReverseMap[TransformIndex0]] + VertexCount[ReverseMap[TransformIndex0]]; i++)
		{
			Vertex[i] += FVector3f(1, 0, 0);
		}
		for (int32 i = VertexStart[ReverseMap[TransformIndex1]]; i < VertexStart[ReverseMap[TransformIndex1]] + VertexCount[ReverseMap[TransformIndex1]]; i++)
		{
			Vertex[i] -= FVector3f(1, 0, 0);
		}

		TManagedArray<FString> & Names = Collection->BoneName;
		Names[ParentIndex] = "Root";
		Names[TransformIndex0] = "RGB1";
		Names[TransformIndex1] = "RGB2";

		TManagedArray<int32> & Parents = Collection->Parent;
		TManagedArray<TSet<int32>>& Children = Collection->Children;

		Parents[ParentIndex] = FGeometryCollection::Invalid;
		Children[ParentIndex].Add(TransformIndex0);
		Children[ParentIndex].Add(TransformIndex1);
		Parents[TransformIndex0] = ParentIndex;
		Parents[TransformIndex1] = ParentIndex;
	}


	void SetupNestedBoneCollection(FGeometryCollection* Collection)
	{
		int32 TransformIndex0 = Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0, 0, 90.)), FVector(0, 10, 0)), FVector(1.0)));
		int32 TransformIndex1 = Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0, 0, 45.)), FVector(0, 10, 0)), FVector(1.0)));
		int32 TransformIndex2 = Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0, 0, 45.)), FVector(0, 10, 0)), FVector(1.0)));
		int32 TransformIndex3 = Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(90.,0., 0)), FVector(0, 10, 0)), FVector(1.0)));
		int32 TransformIndex4 = Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(45.,45., 45.)), FVector(0, 10, 0)), FVector(1.0)));

		//  0
		//  ...1
		//  ......2
		//  .........3
		//  ............4
		(Collection->Parent)[TransformIndex0] = FGeometryCollection::Invalid;
		(Collection->Children)[TransformIndex0].Add(TransformIndex1);
		(Collection->Parent)[TransformIndex1] = TransformIndex0;
		(Collection->Children)[TransformIndex1].Add(TransformIndex2);
		(Collection->Parent)[TransformIndex2] = TransformIndex1;
		(Collection->Children)[TransformIndex2].Add(TransformIndex3);
		(Collection->Parent)[TransformIndex3] = TransformIndex2;
		(Collection->Children)[TransformIndex3].Add(TransformIndex4);
		(Collection->Parent)[TransformIndex4] = TransformIndex3;
	}

	void ComputeInnerAndOuterRadiiFromGeometryVertices(const TManagedArray<FVector3f>& Vertices, const int32 VertexStart, const int32 VertexCount, float& OutInnerRadius, float& OutOuterRadius)
	{
		// first compute the geometry center
		FVector VertexSum{0.0 };
		for (int32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
		{
			VertexSum += FVector(Vertices[VertexStart + VertexIndex]); 
		}
		FVector3f Center{0.};
		if (VertexCount > 0)
		{
			// controlled precision loss, we are now back into the range of vertices
			Center = FVector3f{VertexSum / static_cast<FVector::FReal>(VertexCount)};
		}
		
		OutInnerRadius = VertexCount ? TNumericLimits<float>::Max() : 0.0f;
		OutOuterRadius = 0.0f;
		for (int32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
		{
			const FVector3f& Pt = Vertices[VertexStart + VertexIndex];
			const float DistSq = FVector3f::DistSquared(Pt, Center);
			OutInnerRadius = FMath::Min(OutInnerRadius, DistSq);
			OutOuterRadius = FMath::Max(OutOuterRadius, DistSq);
		}
		OutInnerRadius = FMath::Sqrt(OutInnerRadius);
		OutOuterRadius = FMath::Sqrt(OutOuterRadius);
	}
	
	void AddGeometryProperties(FManagedArrayCollection* InCollection)
	{
		if (InCollection)
		{
			if (!InCollection->NumElements(FGeometryCollection::GeometryGroup))
			{
				int32 NumVertices = InCollection->NumElements(FGeometryCollection::VerticesGroup);
				if (NumVertices)
				{
					// transforms group
					TManagedArray<FTransform3f>& Transform = InCollection->ModifyAttribute<FTransform3f>(FTransformCollection::TransformAttribute, FTransformCollection::TransformGroup);
					TManagedArray<int32>& TransformToGeometryIndex = InCollection->ModifyAttribute<int32>("TransformToGeometryIndex", FTransformCollection::TransformGroup);
					// vertices group
					TManagedArray<int32>& BoneMap = InCollection->ModifyAttribute<int32>("BoneMap", FGeometryCollection::VerticesGroup);
					TManagedArray<FVector3f>& Vertex = InCollection->ModifyAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup);
					// faces
					TManagedArray<FIntVector>& FaceIndices = InCollection->ModifyAttribute<FIntVector>("Indices", FGeometryCollection::FacesGroup);

					// geometry group
					TManagedArray<int32>& TransformIndex = InCollection->ModifyAttribute<int32>("TransformIndex", FGeometryCollection::GeometryGroup);
					TManagedArray<FBox>& BoundingBox = InCollection->ModifyAttribute<FBox>("BoundingBox", FGeometryCollection::GeometryGroup);
					TManagedArray<float>& InnerRadius = InCollection->ModifyAttribute<float>("InnerRadius", FGeometryCollection::GeometryGroup);
					TManagedArray<float>& OuterRadius = InCollection->ModifyAttribute<float>("OuterRadius", FGeometryCollection::GeometryGroup);
					TManagedArray<int32>& VertexCount = InCollection->ModifyAttribute<int32>("VertexCount", FGeometryCollection::GeometryGroup);
					TManagedArray<int32>& VertexStart = InCollection->ModifyAttribute<int32>("VertexStart", FGeometryCollection::GeometryGroup);
					TManagedArray<int32>& FaceCount = InCollection->ModifyAttribute<int32>("FaceCount", FGeometryCollection::GeometryGroup);
					TManagedArray<int32>& FaceStart = InCollection->ModifyAttribute<int32>("FaceStart", FGeometryCollection::GeometryGroup);

					// gather unique geometries
					TSet<int32> TransformIndexOfGeometry;
					for (int32 BoneIdx = 0; BoneIdx < BoneMap.Num(); BoneIdx++)
					{
						TransformIndexOfGeometry.Add(BoneMap[BoneIdx]);
					}

					// reverse map
					TArray<int32> ReverseMap;
					ReverseMap.Init(FGeometryCollection::Invalid, Transform.Num());

					InCollection->AddElements(TransformIndexOfGeometry.Num(), FGeometryCollection::GeometryGroup);
					TArray<int32> GeometryIndices = TransformIndexOfGeometry.Array();
					for (int32 Index = 0; Index < GeometryIndices.Num(); Index++)
					{
						ReverseMap[GeometryIndices[Index]] = Index;
						TransformToGeometryIndex[GeometryIndices[Index]] = Index;

						TransformIndex[Index] = GeometryIndices[Index];
						BoundingBox[Index].Init();
						InnerRadius[Index] = FLT_MAX;
						OuterRadius[Index] = -FLT_MAX;
						VertexStart[Index] = FGeometryCollection::Invalid;
						VertexCount[Index] = 0;
						FaceStart[Index] = FGeometryCollection::Invalid;
						FaceCount[Index] = 0;
					}

					// build vertex summary information
					TArray<FVector3f> Center;
					Center.Init(FVector3f(0), GeometryIndices.Num());
					int CurrentParticleIndex = FGeometryCollection::Invalid;
					for (int vdx = 0; vdx < Vertex.Num(); vdx++)
					{
						int32 ParticleIndex = BoneMap[vdx];
						check(ReverseMap[ParticleIndex] != FGeometryCollection::Invalid);
						int32 GeometryIndex = ReverseMap[ParticleIndex];

						if (VertexStart[GeometryIndex] == FGeometryCollection::Invalid)
						{
							// @todo(ContigiousVertices) : Files on disk are not contiguous, so until they are fixed just use the first set of vertices.
							//ensureMsgf(VertexCount[GeometryIndex] == 0, TEXT("Expected empty vertex count."));

							VertexStart[GeometryIndex] = vdx;
							CurrentParticleIndex = ParticleIndex;
						}
						if (ParticleIndex == CurrentParticleIndex)
						{
							VertexCount[GeometryIndex]++;
							BoundingBox[GeometryIndex] += FVector(Vertex[vdx]);
						}
						// ensure contiguous particle indices
						// @todo(ContigiousVertices) : Files on disk are not contiguous, so until they are fixed just use the first set of vertices.
						// ensureMsgf(ParticleIndex == CurrentParticleIndex, TEXT("Expected contiguous particle indices in rigid body creation."));

						Center[GeometryIndex] += Vertex[vdx];
					}

					// build vertex centers
					for (int GeometryIndex = 0; GeometryIndex < GeometryIndices.Num(); GeometryIndex++)
					{
						if (VertexCount[GeometryIndex])
						{
							Center[GeometryIndex] /= static_cast<Chaos::FRealSingle>(VertexCount[GeometryIndex]);
						}
					}


					// build face summary information
					CurrentParticleIndex = FGeometryCollection::Invalid;
					for (int fdx = 0; fdx < FaceIndices.Num(); fdx++)
					{
						int32 vdx = FaceIndices[fdx][0];
						int32 ParticleIndex = BoneMap[vdx];
						check(ReverseMap[ParticleIndex] != FGeometryCollection::Invalid);
						int32 GeometryIndex = ReverseMap[ParticleIndex];

						if (FaceStart[GeometryIndex] == FGeometryCollection::Invalid)
						{
							// @todo(ContigiousVertices) : Files on disk are not contiguous, so until they are fixed just use the first set of vertices.
							//ensureMsgf(FaceCount[GeometryIndex] == 0, TEXT("Expected empty face count."));

							FaceStart[GeometryIndex] = fdx;
							CurrentParticleIndex = ParticleIndex;
						}
						if (ParticleIndex == CurrentParticleIndex)
						{
							FaceCount[GeometryIndex]++;
						}
						// ensure contiguous particle indices
						// @todo(ContigiousVertices) : Files on disk are not contiguous, so until they are fixed just use the first set of vertices.
						//ensureMsgf(ParticleIndex == CurrentParticleIndex, TEXT("Expected contiguous particle indices in rigid body creation."));
					}

					// find the inner and outer radius
					for (int vdx = 0; vdx < Vertex.Num(); vdx++)
					{
						int32 GeometryIndex = ReverseMap[BoneMap[vdx]]; // double indexing safe due to check in previous loop.
						Chaos::FRealSingle Delta = (Center[GeometryIndex] - Vertex[vdx]).Size();
						InnerRadius[GeometryIndex] = FMath::Min(InnerRadius[GeometryIndex], Delta);
						OuterRadius[GeometryIndex] = FMath::Max(OuterRadius[GeometryIndex], Delta);
					}

					// Inner/Outer centroid
					for (int fdx = 0; fdx <FaceIndices.Num(); fdx++)
					{
						int vdx = FaceIndices[fdx][0];
						int32 GeometryIndex = ReverseMap[BoneMap[vdx]]; // double indexing safe due to check in previous loop.

						FVector3f Centroid(0);
						for (int e = 0; e < 3; e++)
						{
							Centroid += Vertex[FaceIndices[fdx][e]];
						}
						Centroid /= 3.0f;

						float Delta = (Center[GeometryIndex] - Centroid).Size();
						InnerRadius[GeometryIndex] = FMath::Min(InnerRadius[GeometryIndex], Delta);
						OuterRadius[GeometryIndex] = FMath::Max(OuterRadius[GeometryIndex], Delta);
					}

					// Inner/Outer edges
					for (int fdx = 0; fdx < FaceIndices.Num(); fdx++)
					{
						int vdx = FaceIndices[fdx][0];
						int32 GeometryIndex = ReverseMap[BoneMap[vdx]]; // double indexing safe due to check in previous loop.
						for (int e = 0; e < 3; e++)
						{
							int i = e, j = (e + 1) % 3;
							FVector3f Edge = Vertex[FaceIndices[fdx][i]] + 0.5*(Vertex[FaceIndices[fdx][j]] - Vertex[FaceIndices[fdx][i]]);
							float Delta = (Center[GeometryIndex] - Edge).Size();
							InnerRadius[GeometryIndex] = FMath::Min(InnerRadius[GeometryIndex], Delta);
							OuterRadius[GeometryIndex] = FMath::Max(OuterRadius[GeometryIndex], Delta);
						}
					}
				}

				FGeometryCollection::UpdateBoundingBox(*InCollection);
			}
		}
	}

	void MakeMaterialsContiguous(FGeometryCollection * Collection)
	{
		// if the material indices are not setup then they will all be zero, then this is an old asset needing updated
		if (Collection->NumElements(FGeometryCollection::FacesGroup) 
			&& (Collection->MaterialIndex)[0] == (Collection->MaterialIndex)[1] && (Collection->MaterialIndex)[0] == 0)
		{
			int NumVisited = 0;
			// fill in the material IDs
			TManagedArray<FGeometryCollectionSection>& Section = Collection->Sections;
			TManagedArray<int32>& MaterialID = Collection->MaterialID;
			for (int i = 0; i < Section.Num(); i++)
			{
				int first = Section[i].FirstIndex / 3;
				int last = first + Section[i].NumTriangles;

				for (int FaceIdx = first; FaceIdx < last; FaceIdx++)
				{
					MaterialID[FaceIdx] = Section[i].MaterialID;
					NumVisited++;
				}

			}

			check(NumVisited == Collection->NumElements(FGeometryCollection::FacesGroup));

			// Reindex will update everything else that is required
			Collection->ReindexMaterials();
		}
	}


	void GenerateTemporaryGuids(FManagedArrayCollection* Collection, int32 StartIdx, bool bForceInit)
	{
		bool bNeedsInit = false;
		if (!Collection->HasAttribute("GUID", FTransformCollection::TransformGroup))
		{
			FManagedArrayCollection::FConstructionParameters Params(FName(), false);
			Collection->AddAttribute<FGuid>("GUID", FTransformCollection::TransformGroup, Params);
			bNeedsInit = true;
		}

		if (bNeedsInit || bForceInit)
		{
			TManagedArray<FGuid>& Guids = Collection->ModifyAttribute<FGuid>("GUID", FTransformCollection::TransformGroup);
			for (int32 Idx = StartIdx; Idx < Guids.Num(); ++Idx)
			{
				Guids[Idx] = FGuid::NewGuid();
			}
		}
	}

}