// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "Templates/UnrealTemplate.h"
#include "Chaos/ChaosArchive.h"
#include "Serialization/MemoryWriter.h"
#include "UObject/UE5MainStreamObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ManagedArrayCollection)

DEFINE_LOG_CATEGORY_STATIC(FManagedArrayCollectionLogging, NoLogging, All);

int8 FManagedArrayCollection::Invalid = INDEX_NONE;


FManagedArrayCollection::FManagedArrayCollection()
{
	Version = 9;
}


FManagedArrayCollection::~FManagedArrayCollection() = default;

FManagedArrayCollection::FManagedArrayCollection(const FManagedArrayCollection& In) { In.CopyTo(this); }
FManagedArrayCollection& FManagedArrayCollection::operator=(const FManagedArrayCollection& In) { Reset(); In.CopyTo(this); return *this; }
FManagedArrayCollection::FManagedArrayCollection(FManagedArrayCollection&&) = default;
FManagedArrayCollection& FManagedArrayCollection::operator=(FManagedArrayCollection&&)= default;

bool FManagedArrayCollection::operator==(const FManagedArrayCollection& Other) const
{
	if (&Other == this || (Other.Map.IsEmpty() && Map.IsEmpty()))
	{
		return true;
	}
	if (Other.Map.Num() != Map.Num())
	{
		return false;
	}
	for (const TTuple<FKeyType, FValueType>& Entry : Map)
	{
		const FName& AttributeName = Entry.Key.Get<0>();
		const FName& GroupName = Entry.Key.Get<1>();
		if (!Other.HasAttribute(AttributeName, GroupName))
		{
			return false;
		}
		if (Other.NumElements(GroupName) != NumElements(GroupName))
		{
			return false;
		}
	}
	auto GetSerializedData = [](const FManagedArrayCollection& ManagedArrayCollection) -> TArray<uint8>
		{
			TArray<uint8> SerializedData;
			FMemoryWriter Ar(SerializedData);
			const_cast<FManagedArrayCollection&>(ManagedArrayCollection).Serialize(Ar);
			Ar.Close();
			return SerializedData;
		};
	return GetSerializedData(*this) == GetSerializedData(Other);
}

void FManagedArrayCollection::AddGroup(FName Group)
{
	ensure(!GroupInfo.Contains(Group));
	FGroupInfo info{
		0
	};
	GroupInfo.Add(Group, info);
}

int32 FManagedArrayCollection::NumAttributes(FName Group) const
{
	int32 Num=0;
	for (const TTuple<FKeyType, FValueType>& Entry : Map)
	{
		if (Entry.Key.Get<1>() == Group)
		{
			Num++;
		}
	}
	return Num;
}

void FManagedArrayCollection::RemoveElements(const FName& Group, const TArray<int32>& SortedDeletionList, FProcessingParameters Params)
{
	if (SortedDeletionList.Num())
	{
		int32 GroupSize = NumElements(Group);
		int32 DelListNum = SortedDeletionList.Num();
		GeometryCollectionAlgo::ValidateSortedList(SortedDeletionList, GroupSize);
		ensure(GroupSize >= DelListNum);

		TArray<int32> Offsets;
		GeometryCollectionAlgo::BuildIncrementMask(SortedDeletionList, GroupSize, Offsets);

		TSet<int32> DeletionSet(SortedDeletionList);
		for (TTuple<FKeyType, FValueType>& Entry : Map)
		{
			//
			// Reindex attributes dependent on the group being resized
			//
			if (Entry.Value.GetGroupIndexDependency() == Group && Params.bReindexDependentAttibutes)
			{
				Entry.Value.Modify().Reindex(Offsets, GroupSize - DelListNum, SortedDeletionList, DeletionSet);
			}

			//
			//  Resize the array and clobber deletion indices
			//
			if (Entry.Key.Get<1>() == Group)
			{
				Entry.Value.Modify().RemoveElements(SortedDeletionList);
			}

		}
		GroupInfo[Group].Size -= DelListNum;
	}
}

void FManagedArrayCollection::MergeElements(const FName& Group, const TArray<int32>& SortedMergeList, const TArray<int32>& MergeRemapIndex, FProcessingParameters Params)
{
	check(SortedMergeList.Num() == MergeRemapIndex.Num());
	if (SortedMergeList.Num())
	{
		const int32 GroupSize = GroupInfo[Group].Size;
		TArray<int32> InverseNewOrder;
		InverseNewOrder.Init(INDEX_NONE, GroupSize);
		for (int32 Idx = 0; Idx < GroupSize; ++Idx)
		{
			InverseNewOrder[Idx] = Idx;
		}
		for (int32 DeletedIdx = 0; DeletedIdx < MergeRemapIndex.Num(); ++DeletedIdx)
		{
			InverseNewOrder[SortedMergeList[DeletedIdx]] = MergeRemapIndex[DeletedIdx];
		}
		for (TTuple<FKeyType, FValueType>& Entry : Map)
		{
			// Reindex attributes dependent deleted elements
			if (Entry.Value.GetGroupIndexDependency() == Group)
			{
				Entry.Value.Modify().ReindexFromLookup(InverseNewOrder);
			}
		}
		FManagedArrayCollection::RemoveElements(Group, SortedMergeList, Params);
	}
}

void FManagedArrayCollection::RemoveElements(const FName& Group, int32 NumberElements, int32 Position)
{
	TArray<int32> SortedDeletionList;
	SortedDeletionList.SetNumUninitialized(NumberElements);
	for (int32 Idx = 0; Idx < NumberElements; ++Idx)
	{
		SortedDeletionList[Idx] = Position + Idx;
	}
	RemoveElements(Group, SortedDeletionList);
}

TArray<FName> FManagedArrayCollection::GroupNames() const
{
	TArray<FName> keys;
	if (GroupInfo.Num())
	{
		GroupInfo.GetKeys(keys);
	}
	return keys;
}

bool FManagedArrayCollection::HasAttribute(FName Name, FName Group) const
{
	FKeyType Key = FManagedArrayCollection::MakeMapKey(Name, Group);
	return Map.Contains(Key);
}

bool FManagedArrayCollection::HasAttributes(const TArray<FManagedArrayCollection::FManagedType>& Types) const
{
	for (const FManagedType& ManagedType : Types)
	{
		if(const FValueType* FoundValue = Map.Find({ ManagedType.Name, ManagedType.Group }))
		{
			if (FoundValue->GetArrayType() != ManagedType.Type)
			{
				return false;
			}
		}
		else
		{
			return false;
		}
	}
	return true;
}

FManagedArrayCollection::EArrayType FManagedArrayCollection::GetAttributeType(FName Name, FName Group) const
{
	if (const FValueType* Attribute = Map.Find({ Name, Group }))
	{
		return Attribute->GetArrayType();
	}
	return EArrayType::FNoneType;
}

bool FManagedArrayCollection::IsAttributeDirty(FName Name, FName Group) const
{
	if (const FValueType* Attribute = Map.Find({ Name, Group }))
	{
		return Attribute->IsDirty();
	}
	return false;
}

bool FManagedArrayCollection::IsAttributePersistent(FName Name, FName Group) const
{
	if (const FValueType* Attribute = Map.Find({ Name, Group }))
	{
		return Attribute->IsPersistent();
	}
	return false;
}

TArray<FName> FManagedArrayCollection::AttributeNames(FName Group) const
{
	TArray<FName> AttributeNames;
	for (const TTuple<FKeyType, FValueType>& Entry : Map)
	{
		if (Entry.Key.Get<1>() == Group)
		{
			AttributeNames.Add(Entry.Key.Get<0>());
		}
	}
	return AttributeNames;
}

int32 FManagedArrayCollection::NumElements(FName GroupName) const
{
	int32 Num = 0;
	if (const FGroupInfo* Group = GroupInfo.Find(GroupName))
	{
		Num = Group->Size;
	}
	return Num;
}

int32 FManagedArrayCollection::AddElements(int32 NumberElements, FName Group)
{
	int32 StartSize = 0;
	if (!GroupInfo.Contains(Group))
	{
		AddGroup(Group);
	}

	StartSize = GroupInfo[Group].Size;
	for (TTuple<FKeyType, FValueType>& Entry : Map)
	{
		if (Entry.Key.Get<1>() == Group)
		{
			Entry.Value.Resize(StartSize + NumberElements);
		}
	}
	GroupInfo[Group].Size += NumberElements;

	SetDefaults(Group, StartSize, NumberElements);

	return StartSize;
}

int32 FManagedArrayCollection::InsertElements(int32 NumberElements, int32 Position, FName Group)
{
	const int32 OldGroupSize = AddElements(NumberElements, Group);
	const int32 NewGroupSize = OldGroupSize + NumberElements;
	check(Position <= OldGroupSize);
	const int32 NumberElementsToMove = OldGroupSize - Position;
	const int32 MoveToPosition = Position + NumberElements;

	TArray<int32> NewOrder;
	NewOrder.SetNumUninitialized(NewGroupSize);

	for (int32 Idx = 0; Idx < Position; ++Idx)
	{
		NewOrder[Idx] = Idx;
	}
	for (int32 Idx = Position; Idx < MoveToPosition; ++Idx)
	{
		NewOrder[Idx] = Idx + NumberElementsToMove;
	}
	for (int32 Idx = MoveToPosition; Idx < NewGroupSize; ++Idx)
	{
		NewOrder[Idx] = Idx - NumberElements;
	}

	ReorderElements(Group, NewOrder);

	return Position;
}

TArray<int32> FManagedArrayCollection::InsertElementsNoReorder(int32 NumberElements, int32 Position, FName Group)
{
	const int32 OldGroupSize = AddElements(NumberElements, Group);
	const int32 NewGroupSize = OldGroupSize + NumberElements;
	check(Position <= OldGroupSize);
	const int32 NumberElementsToMove = OldGroupSize - Position;
	const int32 MoveToPosition = Position + NumberElements;

	TArray<int32> NewOrder;
	NewOrder.SetNumUninitialized(NewGroupSize);

	for (int32 Idx = 0; Idx < Position; ++Idx)
	{
		NewOrder[Idx] = Idx;
	}
	for (int32 Idx = Position; Idx < MoveToPosition; ++Idx)
	{
		NewOrder[Idx] = Idx + NumberElementsToMove;
	}
	for (int32 Idx = MoveToPosition; Idx < NewGroupSize; ++Idx)
	{
		NewOrder[Idx] = Idx - NumberElements;
	}
	return NewOrder;
}

void FManagedArrayCollection::Append(const FManagedArrayCollection& InCollection)
{
	bool bMatchingAttributes = true;
	for (const TTuple<FKeyType, FValueType>& Entry : InCollection.Map)
	{
		if (HasAttribute(Entry.Key.Get<0>(), Entry.Key.Get<1>()))
		{
			const FValueType& OriginalValue = InCollection.Map[Entry.Key];
			const FValueType& DestValue = Map[Entry.Key];

			// If we don't have a type match don't attempt the copy.
			if (OriginalValue.GetArrayType() != DestValue.GetArrayType())
			{
				bMatchingAttributes = false;
				ensureMsgf(false, TEXT("Failed : Type error in FManagedArrayCollection::AppendCollection (%s:%s)"), 
					*Entry.Key.Get<0>().ToString(), *Entry.Key.Get<1>().ToString());
			}
		}
	}
	if (bMatchingAttributes)
	{
		TMap<FName, TArray<int32>> GroupNewOrder;
		// make space first. 
		for (const FName& Group : InCollection.GroupNames())
		{
			if (HasGroup(Group) && InCollection.NumElements(Group))
			{
				GroupNewOrder.Add(Group, InsertElementsNoReorder(InCollection.NumElements(Group), 0, Group));
			}
			else if (!HasGroup(Group))
			{
				AddGroup(Group);
				AddElements(InCollection.NumElements(Group), Group);
			}
		}
		for (TTuple<FName, TArray<int32>>& Pair : GroupNewOrder)
		{
			FManagedArrayCollection::ReorderElements(Pair.Key, Pair.Value);
		}

		// copy values
		for (const TTuple<FKeyType, FValueType>& Entry : InCollection.Map)
		{
			const FName AttributeName = Entry.Key.Get<0>();
			const FName GroupName = Entry.Key.Get<1>();

			if (HasAttribute(AttributeName, GroupName))
			{
				Map[Entry.Key].CopyFrom(Entry.Value);
			}
			else
			{
				FValueType NewAttribute(Entry.Value);
				NewAttribute.Resize(NumElements(GroupName));
				Map.Add(Entry.Key, MoveTemp(NewAttribute));
			}
		}
	}
}


void FManagedArrayCollection::RemoveAttribute(FName Name, FName Group)
{
	FKeyType Key = FManagedArrayCollection::MakeMapKey(Name, Group);
	FValueType* FoundValue = Map.Find(Key);
	if (FoundValue != nullptr)
	{
		// TODO : is that even just necessary only for external values ? 
		// if so maybe we should have this logic inside the empty code of FValueType
		FoundValue->Empty();
	}
	Map.Remove(Key);
}

void FManagedArrayCollection::RemoveGroup(FName Group)
{
	TArray<FName> DelList;
	for (TTuple<FKeyType, FValueType>& Entry : Map)
	{
		if (Entry.Key.Get<1>() == Group)
		{
			DelList.Add(Entry.Key.Get<0>());
		}
		Entry.Value.RemoveGroupIndexDependency(Group);
	}
	for (const FName& AttrName : DelList)
	{
		Map.Remove({ AttrName, Group });
	}

	GroupInfo.Remove(Group);
}

void FManagedArrayCollection::Resize(int32 Size, FName Group)
{
	ensure(HasGroup(Group));
	const int32 CurSize = NumElements(Group);
	if (CurSize == Size)
	{
		return;
	}

	ensureMsgf(Size > NumElements(Group), TEXT("Use RemoveElements to shrink a group."));
	const int32 StartSize = GroupInfo[Group].Size;
	for (TTuple<FKeyType, FValueType>& Entry : Map)
	{
		if (Entry.Key.Get<1>() == Group)
		{
			Entry.Value.Resize(Size);
		}
	}
	GroupInfo[Group].Size = Size;
}

void FManagedArrayCollection::Reserve(int32 Size, FName Group)
{
	ensure(HasGroup(Group));
	const int32 CurSize = NumElements(Group);
	if (CurSize >= Size)
	{
		return;
	}

	const int32 StartSize = GroupInfo[Group].Size;
	for (TTuple<FKeyType, FValueType>& Entry : Map)
	{
		if (Entry.Key.Get<1>() == Group)
		{
			Entry.Value.Reserve(Size);
		}
	}
}

void FManagedArrayCollection::EmptyGroup(FName Group)
{
	ensure(HasGroup(Group));

	for (TTuple<FKeyType, FValueType>& Entry : Map)
	{
		if (Entry.Key.Get<1>() == Group)
		{
			Entry.Value.Empty();
		}
	}

	GroupInfo[Group].Size = 0;
}

void FManagedArrayCollection::ReorderElements(FName Group, const TArray<int32>& NewOrder)
{
	const int32 GroupSize = GroupInfo[Group].Size;
	check(GroupSize == NewOrder.Num());

	TArray<int32> InverseNewOrder;
	InverseNewOrder.Init(INDEX_NONE, GroupSize);
	for (int32 Idx = 0; Idx < GroupSize; ++Idx)
	{
		InverseNewOrder[NewOrder[Idx]] = Idx;
	}

	for (TTuple<FKeyType, FValueType>& Entry : Map)
	{
		// Reindex attributes dependent on the group being reordered
		if (Entry.Value.GetGroupIndexDependency() == Group)
		{
			Entry.Value.Modify().ReindexFromLookup(InverseNewOrder);
		}

		if (Entry.Key.Get<1>() == Group)
		{
			Entry.Value.Modify().Reorder(NewOrder);
		}
	}
}

void FManagedArrayCollection::SetDependency(FName Name, FName Group, FName DependencyGroup, bool bAllowCircularDependency)
{
	ensure(HasAttribute(Name, Group));
	FKeyType Key = FManagedArrayCollection::MakeMapKey(Name, Group);
	if (ensure(bAllowCircularDependency || !IsConnected(DependencyGroup, Group)))
	{
		Map[Key].SetGroupIndexDependency(DependencyGroup);
	}
}

FName FManagedArrayCollection::GetDependency(FName Name, FName Group) const
{
	check(HasAttribute(Name, Group));
	const FKeyType Key = FManagedArrayCollection::MakeMapKey(Name, Group);
	return Map[Key].GetGroupIndexDependency();
}

void FManagedArrayCollection::RemoveDependencyFor(FName Group)
{
	ensure(HasGroup(Group));
	for (TTuple<FKeyType, FValueType>& Entry : Map)
	{
		Entry.Value.RemoveGroupIndexDependency(Group);
	}
}

void FManagedArrayCollection::SetDefaults(FName Group, uint32 StartSize, uint32 NumElements)
{
	for (TTuple<FKeyType, FValueType>& Entry : Map)
	{
		if (Entry.Key.Get<1>() == Group)
		{
			Entry.Value.Modify().SetDefaults(StartSize, NumElements, !Entry.Value.GetGroupIndexDependency().IsNone());
		}
	}
}

void FManagedArrayCollection::SyncGroupSizeFrom(const FManagedArrayCollection& InCollection, FName Group)
{
	SyncGroupSizeFrom(InCollection, Group, Group);
}

void FManagedArrayCollection::SyncGroupSizeFrom(const FManagedArrayCollection& InCollection, FName SrcGroup, FName DstGroup)
{
	if (!HasGroup(DstGroup))
	{
		AddGroup(DstGroup);
	}

	Resize(InCollection.GroupInfo[SrcGroup].Size, DstGroup);
}

void FManagedArrayCollection::CopyMatchingAttributesFrom(const FManagedArrayCollection& FromCollection, const TArrayView<const FAttributeAndGroupId> SkipList)
{
	MatchOptionalDefaultAttributes(FromCollection);

	// we only want to resize the groups that are in common 
	for (const TPair<FName, FGroupInfo>& Pair: FromCollection.GroupInfo)
	{
		const FName& GroupName = Pair.Key;
		if (HasGroup(GroupName))
		{
			Resize(Pair.Value.Size, GroupName);
		}
	}

	for (TTuple<FKeyType, FValueType>& Entry : Map)
	{
		const FName& AttributeName = Entry.Key.Get<0>();
		const FName& GroupName = Entry.Key.Get<1>();

		if (SkipList.Contains(FAttributeAndGroupId{ AttributeName, GroupName }))
		{
			continue;
		}
		if (const FValueType* FromAttribute = FromCollection.Map.Find({ AttributeName, GroupName }))
		{
			FValueType& ToAttribute = Entry.Value;
			if (ToAttribute.GetArrayType() == FromAttribute->GetArrayType())
			{
				ToAttribute.InitFrom(*FromAttribute);
			}
		}
	}
}

void FManagedArrayCollection::CopyMatchingAttributesFrom(
	const FManagedArrayCollection& InCollection,
	const TMap<FName, TSet<FName>>* SkipList)
{
	MatchOptionalDefaultAttributes(InCollection);

	for (const auto& Pair : InCollection.GroupInfo)
	{
		SyncGroupSizeFrom(InCollection, Pair.Key);
	}
	for (TTuple<FKeyType, FValueType>& Entry : Map)
	{
		if (SkipList)
		{
			if (const TSet<FName>* Attrs = SkipList->Find(Entry.Key.Get<1>()))
			{
				if (Attrs->Contains(Entry.Key.Get<0>()))
				{
					continue;
				}
			}
		}
		if (InCollection.HasAttribute(Entry.Key.Get<0>(), Entry.Key.Get<1>()))
		{
			const FValueType& OriginalValue = InCollection.Map[Entry.Key];
			const FValueType& DestValue = Map[Entry.Key];

			// If we don't have a type match don't attempt the copy.
			if(OriginalValue.GetArrayType() == DestValue.GetArrayType())
			{
				CopyAttribute(InCollection, Entry.Key.Get<0>(), Entry.Key.Get<1>());
			}
		}
	}
}

void FManagedArrayCollection::CopyAttribute(const FManagedArrayCollection& InCollection, FName Name, FName Group)
{
	CopyAttribute(InCollection, /*SrcName=*/Name, /*DestName=*/Name, Group, Group);
}

void FManagedArrayCollection::CopyAttribute(const FManagedArrayCollection& InCollection, FName SrcName, FName DestName, FName Group)
{
	CopyAttribute(InCollection, SrcName, DestName, Group, Group);
}

void FManagedArrayCollection::CopyAttribute(const FManagedArrayCollection& InCollection, FName SrcName, FName DestName, FName SrcGroup, FName DstGroup)
{
	SyncGroupSizeFrom(InCollection, SrcGroup, DstGroup);
	FKeyType SrcKey = FManagedArrayCollection::MakeMapKey(SrcName, SrcGroup);
	FKeyType DestKey = FManagedArrayCollection::MakeMapKey(DestName, DstGroup);

	if (!HasAttribute(DestName, DstGroup))
	{
		FValueType NewAttribute(InCollection.Map[SrcKey]);
		Map.Add(DestKey, MoveTemp(NewAttribute));
	}

	const FValueType& OriginalValue = InCollection.Map[SrcKey];
	Map[DestKey].InitFrom(OriginalValue);
}

void FManagedArrayCollection::CopyTo(FManagedArrayCollection* Collection, const TArray<FName>& GroupsToSkip, TArray<TTuple<FName, FName>> AttributesToSkip) const
{
	Collection->Version = Version;

	TArray<FKeyType> AttributesToCopy;
	AttributesToCopy.Reserve(Map.Num());

	// Find all the attributes to copy
	for (const TTuple<FKeyType, FValueType>& Attribute : Map)
	{
		const FName& AttributeName = Attribute.Key.Get<0>();
		const FName& GroupName = Attribute.Key.Get<1>();
		if (AttributesToSkip.Num() > 0)
		{
			const TTuple<FName, FName> GroupAndAttribute{ AttributeName, GroupName };
			if (AttributesToSkip.Contains(GroupAndAttribute))
			{
				continue;
			}
		}
		if (!GroupsToSkip.Contains(GroupName))
		{
			AttributesToCopy.Add(Attribute.Key);
		}
	}

	// Add missing groups and attributes in the destination collection 
	for (const FKeyType& AttributeKey : AttributesToCopy)
	{
		// Create group is missing and resize it to fit the source collection (this)
		const FName& GroupName = AttributeKey.Get<1>();
		if (!Collection->HasGroup(GroupName))
		{
			Collection->AddGroup(GroupName);
		}
		if (NumElements(GroupName) != Collection->NumElements(GroupName))
		{
			ensure(!Collection->NumElements(GroupName));
			Collection->AddElements(NumElements(GroupName), GroupName);
		}

		// Add attribute if missing 
		// no data copy is made, the attribute is only referenced via a shared pointer
		const FName& AttributeName = AttributeKey.Get<0>();
		if (!Collection->HasAttribute(AttributeName, GroupName))
		{
			FValueType NewAttribute(this->Map[AttributeKey]);
			Collection->Map.Add(AttributeKey, MoveTemp(NewAttribute));
		}
	}

	// now that all structures are in place we can paralellize copy of the data
	// FindChecked (rather than operator[]) makes explicit that the lambda only looks up existing entries
	constexpr int32 MinBatch = 1;
	ParallelFor(TEXT("ManagedArrayCollection::CopyTo::DataCopy"), AttributesToCopy.Num(), MinBatch,
		[&AttributesToCopy, &SourceMap = this->Map, &DestMap = Collection->Map](const int32 AttributeIndex)
		{
			const FKeyType& AttributeKey = AttributesToCopy[AttributeIndex];
			DestMap.FindChecked(AttributeKey).InitFrom(SourceMap.FindChecked(AttributeKey));
		});
}

bool FManagedArrayCollection::IsConnected(FName StartingNode, FName TargetNode)
{
	if (!StartingNode.IsNone())
	{
		TMap<FName, TArray<FName> > DMap;
		for (const TTuple<FKeyType, FValueType>& Entry : Map)
		{
			if (!DMap.Contains(Entry.Key.Get<1>()))
				DMap.Add(Entry.Key.Get<1>(), TArray<FName>());
			if (!Entry.Value.GetGroupIndexDependency().IsNone())
				DMap[Entry.Key.Get<1>()].AddUnique(Entry.Value.GetGroupIndexDependency());
		}

		if (DMap.Contains(StartingNode))
		{
			TSet<FName> Visited;
			TArray<FName> SearchSet = DMap[StartingNode];
			while (SearchSet.Num())
			{
				FName Curr = SearchSet.Pop();
				if (Curr.IsEqual(TargetNode))
				{
					return true;
				}

				if (!Visited.Contains(Curr))
				{
					Visited.Add(Curr);
					if (DMap.Contains(Curr))
					{
						if (!DMap[Curr].IsEmpty())
							SearchSet.Append(DMap[Curr]);
					}
				}
			}
		}
	}
	return false;
}

FString FManagedArrayCollection::ToString() const
{
	FString Buffer;

	const TArray<FStringFormatArg> CollectionInfos = { FString::FormatAsNumber((int32)GetAllocatedSize()) };
	Buffer += FString::Format(TEXT("All attributes [{0} bytes]\n"), CollectionInfos);

	for (FName GroupName : GroupNames())
	{
		const TArray<FStringFormatArg> GroupNameInfos = { GroupName.ToString(), FString::FormatAsNumber((int32)NumElements(GroupName)) };
		Buffer += FString::Format(TEXT("{0} - [{1} elements]\n"), GroupNameInfos);
		for (FName AttributeName : AttributeNames(GroupName))
		{
			FKeyType Key = FManagedArrayCollection::MakeMapKey(AttributeName, GroupName);
			const FValueType& Value = Map[Key];

			const SIZE_T AttributeAllocatedSize = Value.Get().GetAllocatedSize();
			const FString AttributeAllocatedSizeStr = FString::FormatAsNumber((int32)AttributeAllocatedSize);

			const TArray<FStringFormatArg> AttributeInfos = { AttributeName.ToString(), AttributeAllocatedSizeStr };
			Buffer += FString::Format(TEXT(" |-- {0} [{1} bytes]\n"), AttributeInfos);
		}
	}
	return Buffer;
}

SIZE_T FManagedArrayCollection::GetAllocatedSize() const
{
	SIZE_T AllocatedSize = Map.GetAllocatedSize();
	for (const TTuple<FKeyType, FValueType>& Entry : Map)
	{
		AllocatedSize += Entry.Value.Get().GetAllocatedSize();
	}
	return AllocatedSize;
}

void FManagedArrayCollection::GetElementSizeInfoForGroups(TArray<TPair<FName, SIZE_T>>& OutSizeInfo) const
{
	// Group name to total element size map
	TMap<FName, SIZE_T> GroupToElementSizeMap;

	for (const TPair<FKeyType, FValueType>& Attribute : Map)
	{
		SIZE_T& GroupSize = GroupToElementSizeMap.FindOrAdd(Attribute.Key.Get<1>());
		GroupSize += Attribute.Value.Get().GetTypeSize();
	}

	for (const TPair<FName, SIZE_T>& GroupAndElementSize : GroupToElementSizeMap)
	{
		OutSizeInfo.Add(GroupAndElementSize);
	}
}

static const FName GuidName("GUID");

// this is a reference wrapper to avoid copying attributes as some may not support copy (unique ptr ones) 
// this is used during serialization to build a transient filtered map of attributes that can be saved
struct FManagedArrayCollectionValueTypeWrapper
{
	FManagedArrayCollectionValueTypeWrapper()
		: ValueRef(nullptr)
	{}

	FManagedArrayCollectionValueTypeWrapper(FManagedArrayCollection::FValueType* ValueRefIn)
		: ValueRef(ValueRefIn)
	{}

	FManagedArrayCollection::FValueType* ValueRef;
};
FArchive& operator<<(FArchive& Ar, FManagedArrayCollectionValueTypeWrapper& ValueIn)
{
	// simple forwarding to the original object 
	Ar << (*ValueIn.ValueRef);
	return Ar;
}

void FManagedArrayCollection::Serialize(Chaos::FChaosArchive& Ar)
{
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);

	Ar << Version;

	if (Ar.IsCountingMemory())
	{
		Ar << GroupInfo;
		Ar << Map;
	}
	else if (Ar.IsLoading())
	{
		//We can't serialize entire tmap in place because we may have new groups. todo(ocohen): baked data should be simpler since all entries exist
		TMap< FName, FGroupInfo> TmpGroupInfo;
		Ar << TmpGroupInfo;

		for (TTuple<FName, FGroupInfo>& Group : TmpGroupInfo)
		{
			GroupInfo.Add(Group);
		}

		//We can't serialize entire tmap in place because some entries may have changed types or memory ownership (internal vs external).
		//todo(ocohen): baked data should be simpler since all entries are guaranteed to exist
		TMap< FKeyType, FValueType> TmpMap;
		Ar << TmpMap;

		for (TTuple<FKeyType, FValueType>& Pair : TmpMap)
		{
			if (FValueType* Existing = Map.Find(Pair.Key))
			{
				if (Existing->GetArrayType() == Pair.Value.GetArrayType())
				{
					Existing->Exchange(Pair.Value);	//if there is already an entry do an exchange. This way external arrays get correct serialization
					//question: should we validate if group dependency has changed in some invalid way?
				}
				else
				{
					Existing->Convert(Pair.Value);
				}
			}
			else
			{
				//todo(ocohen): how do we remove old values? Maybe have an unused attribute concept
				//no existing entry so it is owned by the map
				Map.Add(Pair.Key, MoveTemp(Pair.Value));
			}
		}

		TArray<FKeyType> ToRemoveKeys;
		for (const TTuple<FKeyType, FValueType>& Pair : Map)
		{
			if (!Pair.Value.IsExternal() && !TmpMap.Find(Pair.Key))
			{
				ToRemoveKeys.Add(Pair.Key);
			}
		}
		for (const FKeyType& Key : ToRemoveKeys)
		{
			Map.Remove(Key);
		}

		//it's possible new entries have been added but are not in old content. Resize these.
		for (TTuple<FKeyType, FValueType>& Pair : Map)
		{
			const int32 GroupSize = GroupInfo[Pair.Key.Get<1>()].Size;
			if (GroupSize != Pair.Value.Get().Num())
			{
				Pair.Value.Resize(GroupSize);
			}
		}

		// strip out GUID Attributes
		for (auto& GroupName : GroupNames())
		{
			if (HasAttribute(GuidName, GroupName))
			{
				RemoveAttribute(GuidName, GroupName);
			}
		}

	}
	else // Ar.IsSaving()
	{
		Ar << GroupInfo;
		// Unless it's an undo/redo transaction, strip out the keys that we don't want to save
		if (!Ar.IsTransacting())
		{
			// we do create a wrapper around ValueType, to avoid copies and save memory 
			TMap<FKeyType, FManagedArrayCollectionValueTypeWrapper> ToSaveMap;
			for (TTuple<FKeyType, FValueType>& Pair : Map)
			{
				if (Pair.Value.IsPersistent())
				{
					ToSaveMap.Emplace(Pair.Key, FManagedArrayCollectionValueTypeWrapper(&Pair.Value));
				}
			}
			Ar << ToSaveMap;
		}
		else
		{
			Ar << Map;
		}
	}
}

bool FManagedArrayCollection::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	if (Ar.IsLoading() && Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::AddManagedArrayCollectionPropertySerialization)
	{
		return false;  // No custom serialization yet, default back to SerializeTaggedProperties
	}
	Chaos::FChaosArchive ChaosArchive(Ar);
	Serialize(ChaosArchive);
	return true;
}

FArchive& operator<<(FArchive& Ar, FManagedArrayCollection::FGroupInfo& GroupInfo)
{
	int Version = 4;
	Ar << Version;
	Ar << GroupInfo.Size;
	return Ar;
}

/**
*   operator<<(FValueType)
*/
FArchive& operator<<(FArchive& Ar, FManagedArrayCollection::FValueType& ValueIn)
{
	ValueIn.Serialize(Ar);
	return Ar;
}

void FManagedArrayCollection::FValueType::Serialize(FArchive& Ar)
{
	int SerializationVersion = 4;	//todo: version per entry is really bloated
	Ar << SerializationVersion;

	int ArrayTypeAsInt = static_cast<int>(ArrayType);
	Ar << ArrayTypeAsInt;
	ArrayType = static_cast<FManagedArrayCollection::EArrayType>(ArrayTypeAsInt);

	if (SerializationVersion < 4)
	{
		int ArrayScopeAsInt;
		Ar << ArrayScopeAsInt;	//assume all serialized old content was for rest collection
	}

	if (SerializationVersion >= 2)
	{
		Ar << GroupIndexDependency;
		Ar << bPersistent;
	}

	if (ManagedArray == nullptr)
	{
		ensure(Ar.IsLoading());
		ManagedArray = NewManagedTypedArray(ArrayType);
		SharedManagedArray = TSharedPtr<FManagedArrayBase, ESPMode::ThreadSafe>(ManagedArray);
	}
	
	// Note: We switched to always saving the value here, and use the Saved flag
	// to remove the property from the overall Map (see FManagedArrayCollection::Serialize above)
	bool bNewSavedBehavior = Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) >= FUE5MainStreamObjectVersion::ManagedArrayCollectionAlwaysSerializeValue;
	if (bNewSavedBehavior || bPersistent)
	{
		ManagedArray->Serialize(static_cast<Chaos::FChaosArchive&>(Ar));
	}
}

FManagedArrayCollection::FValueType::FValueType()
	: ArrayType(EArrayType::FNoneType)
	, GroupIndexDependency(NAME_None)
	, bPersistent(true)
	, bExternalValue(false)
	, SharedManagedArray(nullptr)
	, ManagedArray(nullptr)
{};

FManagedArrayCollection::FValueType::FValueType(const FValueType& Other)
	: ArrayType(Other.ArrayType)
	, GroupIndexDependency(Other.GroupIndexDependency)
	, bPersistent(Other.bPersistent)
	, bExternalValue(false)
	, SharedManagedArray(nullptr)
	, ManagedArray(nullptr)
{
	if (Other.ManagedArray)
	{
		if (Other.bExternalValue)
		{
			ManagedArray = NewManagedTypedArray(ArrayType);
			ManagedArray->Resize(Other.ManagedArray->Num());
			ManagedArray->Init(*Other.ManagedArray);

			SharedManagedArray = TSharedPtr<FManagedArrayBase, ESPMode::ThreadSafe>(ManagedArray);
			bExternalValue = false;
		}
		else
		{
			// we only copy the shared pointer as we don't want to pay for the full copy
			// copy on write will make this unique if necessary
			SharedManagedArray = Other.SharedManagedArray;
			ManagedArray = SharedManagedArray.Get();
		}
	}
};

FManagedArrayCollection::FValueType::FValueType(FValueType&& Other)
	: ArrayType(Other.ArrayType)
	, GroupIndexDependency(Other.GroupIndexDependency)
	, bPersistent(Other.bPersistent)
	, bExternalValue(Other.bExternalValue)
	, SharedManagedArray(nullptr)
	, ManagedArray(nullptr)
{
	if (&Other != this)
	{
		SharedManagedArray = MoveTemp(Other.SharedManagedArray);
		ManagedArray = Other.ManagedArray;
		Other.ManagedArray = nullptr;
		Other.bExternalValue = false;
	}
	check(ManagedArray != nullptr);
}

FManagedArrayCollection::FValueType::~FValueType()
{
	if (bExternalValue)
	{
		check(!SharedManagedArray.IsValid());
	}
}

void FManagedArrayCollection::FValueType::MakeUniqueForWrite()
{
	// IsUnique is performant in that case because we are using NonThreadSafe SharedPtr
	// this is a requirement for CopyOnWrite as we forbid cross-thread CopyOnWrite
	if (SharedManagedArray && !SharedManagedArray.IsUnique())
	{
		check(!IsExternal());

		ManagedArray = NewManagedTypedArray(ArrayType);
		ManagedArray->Resize(SharedManagedArray->Num());
		ManagedArray->Init(*SharedManagedArray);

		SharedManagedArray = TSharedPtr<FManagedArrayBase, ESPMode::ThreadSafe>(ManagedArray);
	}
	check(ManagedArray);
}

FManagedArrayBase& FManagedArrayCollection::FValueType::Modify()
{
	MakeUniqueForWrite();

	ManagedArray->MarkDirty();
	return *ManagedArray;
}

void FManagedArrayCollection::FValueType::Reserve(int32 ReservedSize)
{
	if (ReservedSize > ManagedArray->Max())
	{
		MakeUniqueForWrite();
		ManagedArray->Reserve(ReservedSize);
	}
}

void FManagedArrayCollection::FValueType::Resize(int32 NewSize)
{
	if (NewSize > ManagedArray->Num())
	{
		MakeUniqueForWrite();
		ManagedArray->Resize(NewSize);
	}
}

void FManagedArrayCollection::FValueType::InitFrom(const FManagedArrayCollection::FValueType& Other)
{
	MakeUniqueForWrite();
	if (ArrayType == Other.GetArrayType())
	{
		ManagedArray->Init(Other.Get());
	}
	else
	{
		ManagedArray->Convert(Other.Get());
	}
}

void FManagedArrayCollection::FValueType::Exchange(FValueType& Other)
{
	MakeUniqueForWrite();
	check(ArrayType == Other.ArrayType);
	ManagedArray->ExchangeArrays(*Other.ManagedArray);
}

void FManagedArrayCollection::FValueType::Convert(FValueType& Other)
{
	MakeUniqueForWrite();
	check(ArrayType != Other.ArrayType);
	ManagedArray->Convert(*Other.ManagedArray);
}

void FManagedArrayCollection::FValueType::CopyFrom(const FValueType& Other)
{
	MakeUniqueForWrite();
	check(Other.ManagedArray->Num() <= ManagedArray->Num());
	ManagedArray->CopyRange(*Other.ManagedArray, 0, Other.ManagedArray->Num());
}

void FManagedArrayCollection::FValueType::Empty()
{
	MakeUniqueForWrite();
	ManagedArray->Empty();
}

void FManagedArrayCollection::FValueType::RemoveGroupIndexDependency(FName Group)
{
	if (GroupIndexDependency == Group)
	{
		GroupIndexDependency = NAME_None;
	}
}