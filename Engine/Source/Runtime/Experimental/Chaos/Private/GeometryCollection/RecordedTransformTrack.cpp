// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/RecordedTransformTrack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RecordedTransformTrack)

DEFINE_LOG_CATEGORY_STATIC(LogGeometryCollectionCache, Log, All);

TAutoConsoleVariable<int32> CVarLogCacheReduction(TEXT("p.gc.logcachereduction"), 0, TEXT("Logs amount of data removed from a cache after processing"));


FRecordedTransformTrack FRecordedTransformTrack::ProcessRawRecordedData(const FRecordedTransformTrack& InCache)
{
	FRecordedTransformTrack RecordedData = InCache;

	FArchiveCountMem BeforeAr(nullptr);

	if (CVarLogCacheReduction.GetValueOnAnyThread() != 0)
	{
		UScriptStruct* RecordedTrackStructType = FRecordedTransformTrack::StaticStruct();
		RecordedTrackStructType->SerializeTaggedProperties(BeforeAr, (uint8*)&RecordedData, RecordedTrackStructType, nullptr);
	}

	int32 NumRemovedParticles = 0;
	int32 NumTotalParticles = 0;

	const int32 NumRecords = RecordedData.Records.Num();

	for (int32 FrameIndex = NumRecords - 1; FrameIndex >= 0; --FrameIndex)
	{
		FRecordedFrame& CurrFrame = RecordedData.Records[FrameIndex];

		// Previously disabled particles might get uninitialized transforms set.
		// Resolve this here - setting them to identity.
		for (int32 TransformIndex = 0; TransformIndex < CurrFrame.Transforms.Num(); ++TransformIndex)
		{
			FTransform& TM = CurrFrame.Transforms[TransformIndex];
			if (!TM.IsRotationNormalized() && !CurrFrame.DisabledFlags[TransformIndex])
			{
				TM = FTransform::Identity;
			}
		}

		const int32 NumRawEntries = CurrFrame.Transforms.Num();
		CurrFrame.TransformIndices.Reset(NumRawEntries);
		CurrFrame.TransformIndices.AddUninitialized(NumRawEntries);

		NumTotalParticles += NumRawEntries;

		for (int32 ParticleIndex = 0; ParticleIndex < NumRawEntries; ++ParticleIndex)
		{
			CurrFrame.TransformIndices[ParticleIndex] = ParticleIndex;
		}

		// First frame doesn't need to handle dataset reduction as it needs all the transforms
		if (FrameIndex != 0)
		{
			// Otherwise we strip transforms if they haven't changed since last frame, or if the particle is disabled
			FRecordedFrame& PrevFrame = RecordedData.Records[FrameIndex - 1];

			for (int32 ParticleIndex = NumRawEntries - 1; ParticleIndex >= 0; --ParticleIndex)
			{
				if (CurrFrame.DisabledFlags[ParticleIndex] || CurrFrame.Transforms[ParticleIndex].Equals(PrevFrame.Transforms[ParticleIndex]))
				{
					++NumRemovedParticles;
					CurrFrame.Transforms.RemoveAt(ParticleIndex, EAllowShrinking::No);
					CurrFrame.TransformIndices.RemoveAt(ParticleIndex, EAllowShrinking::No);
				}
			}

			// Get rid of the extra memory we've freed up
			CurrFrame.Transforms.Shrink();
			CurrFrame.TransformIndices.Shrink();
		}

		// Next we map each transform in frames after the first back to their previous transform
		// which is required for playback
		if (FrameIndex < (NumRecords - 1))
		{
			FRecordedFrame& NextFrame = RecordedData.Records[FrameIndex + 1];

			const int32 NumNextActiveIndices = NextFrame.TransformIndices.Num();
			NextFrame.PreviousTransformIndices.Reset(NumNextActiveIndices);
			NextFrame.PreviousTransformIndices.AddUninitialized(NumNextActiveIndices);
			for (int32 Index = NumNextActiveIndices - 1; Index >= 0; --Index)
			{
				int32& NextActiveIndex = NextFrame.TransformIndices[Index];

				CurrFrame.TransformIndices.Find(NextActiveIndex, NextFrame.PreviousTransformIndices[Index]);
			}
		}
	}
	
	FArchiveCountMem AfterAr(nullptr);

	if (CVarLogCacheReduction.GetValueOnAnyThread() != 0)
	{
		UScriptStruct* RecordedTrackStructType = FRecordedTransformTrack::StaticStruct();
		RecordedTrackStructType->SerializeTaggedProperties(AfterAr, (uint8*)&RecordedData, RecordedTrackStructType, nullptr);

		const int32 ArchiveBeforeSize = static_cast<int32>(BeforeAr.GetNum());
		const int32 ArchiveAfterSize = static_cast<int32>(AfterAr.GetNum());

		// Dump data reduction stats.
		FNumberFormattingOptions Opts;
		Opts.SetUseGrouping(true);
		Opts.SetMinimumFractionalDigits(2);
		Opts.SetMaximumFractionalDigits(2);
		UE_LOGF(LogGeometryCollectionCache, Log, "Geometry Collection - Processed Raw Capture");
		UE_LOGF(LogGeometryCollectionCache, Log, "    Begin       : %d Particles", NumTotalParticles);
		UE_LOGF(LogGeometryCollectionCache, Log, "    End         : %d Particles", NumTotalParticles - NumRemovedParticles);
		UE_LOGF(LogGeometryCollectionCache, Log, "    Removed     : %d Particles", NumRemovedParticles);
		UE_LOGF(LogGeometryCollectionCache, Log, "    Size Before : %ls KB", *FText::AsNumber(ArchiveBeforeSize / 1024, &Opts).ToString());
		UE_LOGF(LogGeometryCollectionCache, Log, "    Size After  : %ls KB", *FText::AsNumber(ArchiveAfterSize / 1024, &Opts).ToString());
		UE_LOGF(LogGeometryCollectionCache, Log, "    Saved       : %ls KB", *FText::AsNumber((ArchiveBeforeSize - ArchiveAfterSize) / 1024, &Opts).ToString());
		UE_LOGF(LogGeometryCollectionCache, Log, "    On average %.3f removed per tick.", (float)NumRemovedParticles / (float)NumRecords);
		UE_LOGF(LogGeometryCollectionCache, Log, "-------------------------------------------");
	}

	return RecordedData;
}
	
