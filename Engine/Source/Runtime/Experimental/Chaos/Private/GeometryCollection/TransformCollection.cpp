// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GeometryCollection.cpp: FTransformCollection methods.
=============================================================================*/
#include "GeometryCollection/TransformCollection.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"

DEFINE_LOG_CATEGORY_STATIC(FTransformCollectionLogging, NoLogging, All);
const FName FTransformCollection::TransformGroup = "Transform";
const FName FTransformCollection::TransformAttribute = "Transform";
const FName FTransformCollection::ParentAttribute = "Parent";
const FName FTransformCollection::ChildrenAttribute = "Children";
const FName FTransformCollection::ParticlesAttribute = "Particles";
const FName FTransformCollection::LevelAttribute = "Level";
const FName FTransformCollection::ConvexGroup = "Convex";
const FName FTransformCollection::ConvexHullAttribute = "ConvexHull";
const FName FTransformCollection::BoneNameAttribute = "BoneName";
const FName FTransformCollection::BoneColorAttribute = "BoneColor";
const FName FTransformCollection::BoneHierarchyAttribute = "BoneHierarchy";
const FName FTransformCollection::GeometryIndexAttribute = "TransformToGeometryIndex";

FTransformCollection::FTransformCollection()
	: FManagedArrayCollection()
{
	Construct();
}

FTransformCollection::~FTransformCollection() = default;

void FTransformCollection::DefineTransformSchema(FManagedArrayCollection& InCollection)
{
	// Hierarchy Group
	InCollection.AddAttribute<FTransform3f>(FTransformCollection::TransformAttribute, FTransformCollection::TransformGroup);
	InCollection.AddAttribute<FString>(FTransformCollection::BoneNameAttribute, FTransformCollection::TransformGroup);
	InCollection.AddAttribute<FLinearColor>(FTransformCollection::BoneColorAttribute, FTransformCollection::TransformGroup);
	InCollection.AddAttribute<int32>(FTransformCollection::ParentAttribute, FTransformCollection::TransformGroup);
	InCollection.AddAttribute<TSet<int32>>(FTransformCollection::ChildrenAttribute, FTransformCollection::TransformGroup);
}

void FTransformCollection::Construct()
{
	// Hierarchy Group
	AddExternalAttribute<FTransform3f>(FTransformCollection::TransformAttribute, FTransformCollection::TransformGroup, Transform);
	AddExternalAttribute<FString>(FTransformCollection::BoneNameAttribute, FTransformCollection::TransformGroup, BoneName);
	AddExternalAttribute<FLinearColor>(FTransformCollection::BoneColorAttribute, FTransformCollection::TransformGroup, BoneColor);
	AddExternalAttribute<int32>(FTransformCollection::ParentAttribute, FTransformCollection::TransformGroup, Parent);
	AddExternalAttribute<TSet<int32>>(FTransformCollection::ChildrenAttribute, FTransformCollection::TransformGroup, Children);
}

void FTransformCollection::Reset()
{
	Super::Reset();
	Construct();
}

void FTransformCollection::Serialize(Chaos::FChaosArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.IsLoading())
	{
		// for backwards compatibility convert old BoneHierarchy struct into split out arrays
		const TManagedArray<FGeometryCollectionBoneNode>* BoneHierarchyPtr = FindAttribute<FGeometryCollectionBoneNode>(FTransformCollection::BoneHierarchyAttribute, FTransformCollection::TransformGroup);
		if (BoneHierarchyPtr)
		{
			const TManagedArray<FGeometryCollectionBoneNode>& BoneHierarchy = *BoneHierarchyPtr;

			for (int Idx = 0; Idx < BoneHierarchy.Num(); Idx++)
			{
				Parent[Idx] = BoneHierarchy[Idx].Parent;
				Children[Idx] = BoneHierarchy[Idx].Children;
			}
		}
	}

}

FTransformCollection FTransformCollection::SingleTransform(const FTransform& TransformRoot)
{
	FTransformCollection TransformCollection;
	TransformCollection.AddElements(1, FTransformCollection::TransformGroup);
	TransformCollection.Transform[0] = FTransform3f(TransformRoot);
	TransformCollection.Parent[0] = Invalid;
	return TransformCollection;
}

void FTransformCollection::Append(const FManagedArrayCollection& InCollection)
{
	if (const FTransformCollection* InTypedCollection = InCollection.Cast<FTransformCollection>())
	{
		int32 Offset = InCollection.NumElements(TransformGroup);
		Super::Append(InCollection);

		int32 Size = NumElements(TransformGroup);
		for (int Idx = Offset; Idx < Size; Idx++)
		{
			if (Parent[Idx] != INDEX_NONE)
			{
				Parent[Idx] += Offset;
			}
			//Modify the temporary copied array to avoid directly changing values in the TSet
			//Hash value depends on the value of the elements in the TSet
			//If only the value is changed, Find() will not be able to find the value
			TArray<int32> ChildrenArray = Children[Idx].Array();
			for (int32& Val : ChildrenArray)
			{
				Val += Offset;
			}
			Children[Idx] = TSet<int32>(ChildrenArray);
		}
	}
	else
	{
		Super::Append(InCollection);
	}
}


int32 FTransformCollection::AppendTransform(const FTransformCollection & Element, const FTransform& TransformRoot)
{
	check(Element.NumElements(FTransformCollection::TransformGroup) > 0);
	const TManagedArray<FTransform3f>& ElementTransform = Element.Transform;
	const TManagedArray<FString>& ElementBoneName = Element.BoneName;
	const TManagedArray<FLinearColor>& ElementBoneColor = Element.BoneColor;
	const TManagedArray<int32>& ElementParent = Element.Parent;
	const TManagedArray<TSet<int32>>& ElementChildren = Element.Children;

	int OriginalNumTransforms = NumElements(FTransformCollection::TransformGroup);
	int NumElements = Element.NumElements(FTransformCollection::TransformGroup);
	int FirstNewElement = AddElements(NumElements, FTransformCollection::TransformGroup);
	const FTransform3f TransformRoot3f(TransformRoot);
	for (int Index = 0; Index < NumElements; Index++)
	{
		int ParticleIndex = FirstNewElement + Index;
		TManagedArray<FTransform3f>& Transforms = Transform;
		if (ElementParent[Index] == FTransformCollection::Invalid)
		{
			// is root with additional transform
			Transforms[ParticleIndex] = ElementTransform[Index] * TransformRoot3f;
		}
		else
		{
			Transforms[ParticleIndex] = ElementTransform[Index];
		}
		TManagedArray<FString>& BoneNames = BoneName;
		BoneNames[ParticleIndex] = ElementBoneName[Index];
		TManagedArray<FLinearColor>& BoneColors = BoneColor;
		BoneColors[ParticleIndex] = ElementBoneColor[Index];
		Parent[ParticleIndex] = ElementParent[Index];

		if (Parent[ParticleIndex] != -1)
		{
			Parent[ParticleIndex] += OriginalNumTransforms;
		}

		Children[ParticleIndex].Reset();
		for (int ChildElement : ElementChildren[Index])
		{
			Children[ParticleIndex].Add(ChildElement + OriginalNumTransforms);
		}
	}
	return OriginalNumTransforms;
}

void FTransformCollection::ParentTransforms(const int32 TransformIndex, const int32 ChildIndex)
{
	GeometryCollectionAlgo::ParentTransform(this, TransformIndex, ChildIndex);
}

void FTransformCollection::ParentTransforms(const int32 TransformIndex, const TArray<int32>& SelectedBones)
{
	GeometryCollectionAlgo::ParentTransforms(this, TransformIndex, SelectedBones);
}

void FTransformCollection::UnparentTransform(const int32 ChildIndex)
{
	GeometryCollectionAlgo::UnparentTransform(this, ChildIndex);
}



void FTransformCollection::RelativeTransformation(const int32& Index, const FTransform& LocalOffset)
{
	if (ensureMsgf(Index < NumElements(FTransformCollection::TransformGroup), TEXT("Index out of range.")))
	{
		TManagedArray<TSet<int32>>& ChildrenArray = Children;
		TManagedArray<FTransform3f>& TransformArray = Transform;

		const FTransform3f LocalOffset3f = FTransform3f(LocalOffset);
		if (ChildrenArray[Index].Num())
		{
			
			const FTransform3f LocalOffsetInverse = LocalOffset3f.Inverse();
			for (int32 Child : ChildrenArray[Index])
			{
				TransformArray[Child] = TransformArray[Child] * LocalOffsetInverse;
			}
		}
		TransformArray[Index] = LocalOffset3f * TransformArray[Index];
	}
}

void FTransformCollection::RemoveElements(const FName & Group, const TArray<int32> & SortedDeletionList, FProcessingParameters Params)
{
	if (SortedDeletionList.Num())
	{
		if (Group == FTransformCollection::TransformGroup)
		{
			GeometryCollectionAlgo::ValidateSortedList(SortedDeletionList, NumElements(Group));

			TManagedArray<int32>& ParentArray = Parent;
			TManagedArray<TSet<int32>>& ChildrenArray = Children;
			TManagedArray<FTransform3f>&  LocalTransform = Transform;
			for (int32 Sdx = 0; Sdx < SortedDeletionList.Num(); Sdx++)
			{
				TArray<FTransform> GlobalTransform;
				GeometryCollectionAlgo::GlobalMatrices(Transform, Parent, GlobalTransform);

				int32 Index = SortedDeletionList[Sdx];
				ensure(0 <= Index && Index < ParentArray.Num());

				int32 ParentID = ParentArray[Index];
				ensure(ParentID < ParentArray.Num());

				for (int32 ChildID : ChildrenArray[Index])
				{
					FTransform ParentTransform = FTransform::Identity;

					ParentArray[ChildID] = ParentArray[Index];
					if (ParentID >= 0)
					{
						ensure(!ChildrenArray[ParentID].Find(ChildID));
						ChildrenArray[ParentID].Add(ChildID);
						ParentTransform = GlobalTransform[ParentID].Inverse();
					}

					LocalTransform[ChildID] = FTransform3f(GlobalTransform[ChildID] * ParentTransform);
				}

				if (0 <= ParentID)
				{
					ChildrenArray[ParentID].Remove(Index);
				}
			}

			// reindex
			{
				TArray<int32> Offsets;
				GeometryCollectionAlgo::BuildIncrementMask(SortedDeletionList, ParentArray.Num(), Offsets);

				int32 ArraySize = ParentArray.Num(), OffsetsSize = Offsets.Num();
				int32 FinalSize = ParentArray.Num() - SortedDeletionList.Num();
				for (int32 Index = 0; Index < ArraySize; Index++)
				{
					// remap the parents (-1 === Invalid )
					if (ParentArray[Index] != -1)
						ParentArray[Index] -= Offsets[ParentArray[Index]];
					ensure(-1 <= ParentArray[Index] && ParentArray[Index] <= FinalSize);

					// remap children
					TSet<int32> ChildrenCopy = ChildrenArray[Index];
					ChildrenArray[Index].Empty();
					for (int32 ChildID : ChildrenCopy)
					{
						if (0 <= ChildID && ChildID < OffsetsSize)
						{
							int32 NewChildID = ChildID - Offsets[ChildID];
							if (0 <= NewChildID && NewChildID < FinalSize)
							{
								ChildrenArray[Index].Add(NewChildID);
							}
						}
					}
				}
			}

		}

		Super::RemoveElements(Group, SortedDeletionList, Params);

	}
}

void FTransformCollection::MergeElements(const FName& Group, const TArray<int32>& SortedMergeList, const TArray<int32>& MergeRemapIndex, FProcessingParameters Params)
{
	if (SortedMergeList.Num() && ensure(SortedMergeList.Num() == MergeRemapIndex.Num()))
	{
		if (Group == FTransformCollection::TransformGroup)
		{
			GeometryCollectionAlgo::ValidateSortedList(SortedMergeList, NumElements(Group));

			TManagedArray<int32>& ParentArray = Parent;
			TManagedArray<TSet<int32>>& ChildrenArray = Children;
			TManagedArray<FTransform3f>& LocalTransform = Transform;
			for (int32 Sdx = 0; Sdx < SortedMergeList.Num(); ++Sdx)
			{
				const int32 Index = SortedMergeList[Sdx];
				if (ensure(0 <= Index && Index < ParentArray.Num()))
				{
					const int32 RemapId = MergeRemapIndex[Sdx];
					const int32 ParentID = ParentArray[Index];
					if (ensure(0 <= RemapId && RemapId < ParentArray.Num()))
					{
						for (const int32 ChildID : ChildrenArray[Index])
						{
							ParentArray[ChildID] = RemapId;
							ChildrenArray[RemapId].Add(ChildID);
						}
						if (0 <= ParentID && ParentID < ChildrenArray.Num())
						{
							ChildrenArray[ParentID].Remove(Index);
							ChildrenArray[ParentID].Add(RemapId);
						}
						ParentArray[Index] = INDEX_NONE;
						ChildrenArray[Index].Empty();
					}
				}
			}

			// reindex
			{
				TArray<int32> Offsets;
				GeometryCollectionAlgo::BuildIncrementMask(SortedMergeList, ParentArray.Num(), Offsets);

				const int32 ArraySize = ParentArray.Num(), OffsetsSize = Offsets.Num();
				const int32 FinalSize = ParentArray.Num() - SortedMergeList.Num();
				for (int32 Index = 0; Index < ArraySize; Index++)
				{
					// remap the parents (-1 === Invalid )
					if (ParentArray[Index] != -1)
					{
						ParentArray[Index] -= Offsets[ParentArray[Index]];
					}
					ensure(-1 <= ParentArray[Index] && ParentArray[Index] <= FinalSize);

					// remap children
					TSet<int32> ChildrenCopy = ChildrenArray[Index];
					ChildrenArray[Index].Reset();
					for (const int32 ChildID : ChildrenCopy)
					{
						if (0 <= ChildID && ChildID < OffsetsSize)
						{
							const int32 NewChildID = ChildID - Offsets[ChildID];
							if (0 <= NewChildID && NewChildID < FinalSize)
							{
								ChildrenArray[Index].Add(NewChildID);
							}
						}
					}
				}
			}

		}

		Super::MergeElements(Group, SortedMergeList, MergeRemapIndex, Params);
	}
}
