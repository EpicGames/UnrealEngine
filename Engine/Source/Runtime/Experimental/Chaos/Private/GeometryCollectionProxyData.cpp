// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
GeometryCollectionProxyData.cpp: 
=============================================================================*/

#include "GeometryCollectionProxyData.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/TransformCollection.h"

/*
* FTransformDynamicCollection (FManagedArrayCollection)
*/

FTransformDynamicCollection::FTransformDynamicCollection(const FGeometryCollection* InRestCollection)
	: FManagedArrayCollection()
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	, RestCollection(InRestCollection)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	, bTransformHasChanged(false)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	check(RestCollection != nullptr);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	Construct();
}

FTransformDynamicCollection::FTransformDynamicCollection(TSharedPtr<const FGeometryCollection> InRestCollection)
	: FManagedArrayCollection()
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// To be removed with RestCollection post-deprecation
	, RestCollection(InRestCollection.Get())
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	, RestCollectionShared(InRestCollection)
	, bTransformHasChanged(false)
{
	check(RestCollectionShared);
	Construct();
}

void FTransformDynamicCollection::Construct()
{
	FManagedArrayCollection::FConstructionParameters TransformDependency(FTransformCollection::TransformGroup);

	// Transform Group
	AddExternalAttribute<bool>(FTransformCollection::ParentAttribute, FTransformCollection::TransformGroup, HasParent);
	CopyAttribute(*RestCollectionShared, FTransformCollection::ParentAttribute, FTransformCollection::TransformGroup);
}

void FTransformDynamicCollection::InitializeTransforms()
{
	if (bTransformHasChanged == false)
	{
		AddExternalAttribute<FTransform3f>(FTransformCollection::TransformAttribute, FTransformCollection::TransformGroup, Transform);
		CopyAttribute(*RestCollectionShared, FTransformCollection::TransformAttribute, FTransformCollection::TransformGroup);

		bTransformHasChanged = true;
	}
}

const FTransform3f& FTransformDynamicCollection::GetTransform(int32 Index) const
{
	if (bTransformHasChanged == false)
	{
		return RestCollectionShared->Transform[Index];
	}
	return Transform[Index];
}

void FTransformDynamicCollection::SetTransform(int32 Index, const FTransform3f& InTransform)
{
	InitializeTransforms();
	Transform[Index] = InTransform;
}

int32 FTransformDynamicCollection::GetNumTransforms() const
{
	ensure(!bTransformHasChanged || RestCollectionShared->Transform.Num() == Transform.Num());
	return RestCollectionShared->Transform.Num();
}

void FTransformDynamicCollection::ResetInitialTransforms()
{
	if (bTransformHasChanged)
	{
		RemoveAttribute(FTransformCollection::TransformAttribute, FTransformCollection::TransformGroup);
		bTransformHasChanged = false;
	}
}

const TManagedArray<bool>& FTransformDynamicCollection::GetHasParent() const
{
	return HasParent;
}

bool FTransformDynamicCollection::GetHasParent(int32 Index) const
{
	return HasParent[Index];
}

void FTransformDynamicCollection::SetHasParent(int32 Index, bool Value)
{
	InitializeTransforms();
	HasParent[Index] = Value;
}

int32 FTransformDynamicCollection::GetParent(int32 Index) const 
{
	check(RestCollectionShared);
	return (HasParent.IsValidIndex(Index) && HasParent[Index])  ? RestCollectionShared->Parent[Index] : INDEX_NONE;
}

bool FTransformDynamicCollection::HasChildren(int32 Index) const
{
	const TSet<int32>& Children(RestCollectionShared->Children[Index]);
	for (int32 Child : Children)
	{
		if (HasParent[Child])
		{
			return true;
		}
	}
	return false;
}

bool FTransformDynamicCollection::IsCluster(int32 Index) const
{
	if (RestCollectionShared && RestCollectionShared->Children.IsValidIndex(Index))
	{
		return (RestCollectionShared->Children[Index].Num() > 0);
	}
	return false;
}

/*
* FGeometryDynamicCollection (FTransformDynamicCollection)
*/

const FName FGeometryDynamicCollection::ActiveAttribute("Active");
const FName FGeometryDynamicCollection::DynamicStateAttribute("DynamicState");
const FName FGeometryDynamicCollection::ImplicitsAttribute("Implicits");
const FName FGeometryDynamicCollection::ShapesQueryDataAttribute("ShapesQueryData");
const FName FGeometryDynamicCollection::ShapesSimDataAttribute("ShapesSimData");
const FName FGeometryDynamicCollection::SimplicialsAttribute("CollisionParticles");
const FName FGeometryDynamicCollection::SimulatableParticlesAttribute("SimulatableParticlesAttribute");
const FName FGeometryDynamicCollection::SharedImplicitsAttribute("SharedImplicits");
const FName FGeometryDynamicCollection::InternalClusterParentTypeAttribute("InternalClusterParentTypeArray");

// Deprecated
const FName FGeometryDynamicCollection::CollisionMaskAttribute("CollisionMask");
const FName FGeometryDynamicCollection::CollisionGroupAttribute("CollisionGroup");

FGeometryDynamicCollection::FGeometryDynamicCollection(const FGeometryCollection* InRestCollection)
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	: FTransformDynamicCollection(InRestCollection)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	, OptionalLinearVelocityAttribute(nullptr)
	, OptionalAngularVelocityAttribute(nullptr)
	, OptionalAnimateTransformAttribute(nullptr)
{
	// Transform Group
	AddExternalAttribute<bool>(FGeometryDynamicCollection::ActiveAttribute, FTransformCollection::TransformGroup, Active);
	AddExternalAttribute<uint8>(FGeometryDynamicCollection::DynamicStateAttribute, FTransformCollection::TransformGroup, DynamicState);
	AddExternalAttribute(SimulatableParticlesAttribute, FGeometryCollection::TransformGroup, SimulatableParticles);
	AddExternalAttribute(InternalClusterParentTypeAttribute, FGeometryCollection::TransformGroup, InternalClusterParentType);
}

FGeometryDynamicCollection::FGeometryDynamicCollection(TSharedPtr<const FGeometryCollection> InRestCollection)
	: FTransformDynamicCollection(InRestCollection)
	, OptionalLinearVelocityAttribute(nullptr)
	, OptionalAngularVelocityAttribute(nullptr)
	, OptionalAnimateTransformAttribute(nullptr)
{
	// Transform Group
	AddExternalAttribute<bool>(FGeometryDynamicCollection::ActiveAttribute, FTransformCollection::TransformGroup, Active);
	AddExternalAttribute<uint8>(FGeometryDynamicCollection::DynamicStateAttribute, FTransformCollection::TransformGroup, DynamicState);
	AddExternalAttribute(SimulatableParticlesAttribute, FGeometryCollection::TransformGroup, SimulatableParticles);
	AddExternalAttribute(InternalClusterParentTypeAttribute, FGeometryCollection::TransformGroup, InternalClusterParentType);
}

const TManagedArrayAccessor<int32> FGeometryDynamicCollection::GetInitialLevels() const
{
	static const FName LevelAttributeName = "Level";
	return TManagedArrayAccessor<int32>(*RestCollectionShared, LevelAttributeName, FGeometryCollection::TransformGroup);
}

void FGeometryDynamicCollection::AddVelocitiesAttributes()
{
	if (OptionalLinearVelocityAttribute == nullptr && OptionalAngularVelocityAttribute == nullptr)
	{
		static const FName LinearVelocityAttributeName = "LinearVelocity";
		static const FName AngularVelocityAttributeName = "AngularVelocity";

		OptionalLinearVelocityAttribute = &AddAttribute<FVector3f>(LinearVelocityAttributeName, FTransformCollection::TransformGroup);
		OptionalAngularVelocityAttribute = &AddAttribute<FVector3f>(AngularVelocityAttributeName, FTransformCollection::TransformGroup);
	}
}

void FGeometryDynamicCollection::AddAnimateTransformAttribute()
{
	if (OptionalAnimateTransformAttribute == nullptr)
	{
		static const FName AnimateTransformAttributeName = "AnimateTransformAttribute";

		OptionalAnimateTransformAttribute = &AddAttribute<bool>("AnimateTransformAttribute", FGeometryCollection::TransformGroup);
		if (OptionalAnimateTransformAttribute)
		{
			OptionalAnimateTransformAttribute->Fill(false);
		}
	}
}

void FGeometryDynamicCollection::CopyInitialVelocityAttributesFrom(const FGeometryDynamicCollection& SourceCollection)
{
	FInitialVelocityFacade InitialVelocityFacade(*this);
	InitialVelocityFacade.CopyFrom(SourceCollection);
}

FGeometryDynamicCollection::FInitialVelocityFacade::FInitialVelocityFacade(FGeometryDynamicCollection& DynamicCollection)
	: InitialLinearVelocityAttribute(DynamicCollection, "InitialLinearVelocity", FTransformCollection::TransformGroup)
	, InitialAngularVelocityAttribute(DynamicCollection, "InitialAngularVelocity", FTransformCollection::TransformGroup)
{}

FGeometryDynamicCollection::FInitialVelocityFacade::FInitialVelocityFacade(const FGeometryDynamicCollection& DynamicCollection)
	: InitialLinearVelocityAttribute(DynamicCollection, "InitialLinearVelocity", FTransformCollection::TransformGroup)
	, InitialAngularVelocityAttribute(DynamicCollection, "InitialAngularVelocity", FTransformCollection::TransformGroup)
{}

bool FGeometryDynamicCollection::FInitialVelocityFacade::IsValid() const
{
	return InitialLinearVelocityAttribute.IsValid() && InitialAngularVelocityAttribute.IsValid();
}

void FGeometryDynamicCollection::FInitialVelocityFacade::DefineSchema()
{
	InitialLinearVelocityAttribute.Add();
	InitialAngularVelocityAttribute.Add();
}

void FGeometryDynamicCollection::FInitialVelocityFacade::Fill(const FVector3f& InitialLinearVelocity, const FVector3f& InitialAngularVelocity)
{
	check(IsValid());
	InitialLinearVelocityAttribute.Fill(InitialLinearVelocity);
	InitialAngularVelocityAttribute.Fill(InitialAngularVelocity);
}

void FGeometryDynamicCollection::FInitialVelocityFacade::CopyFrom(const FGeometryDynamicCollection& SourceCollection)
{
	FInitialVelocityFacade SourceInitialVelocityFacade(SourceCollection);
	if (SourceInitialVelocityFacade.IsValid())
	{
		DefineSchema();
		InitialLinearVelocityAttribute.Copy(SourceInitialVelocityFacade.InitialLinearVelocityAttribute);
		InitialAngularVelocityAttribute.Copy(SourceInitialVelocityFacade.InitialAngularVelocityAttribute);
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FGeometryCollectionDynamicStateFacade::FGeometryCollectionDynamicStateFacade(FGeometryDynamicCollection& InCollection)
	: ActiveAttribute(InCollection, FGeometryDynamicCollection::ActiveAttribute,  FTransformCollection::TransformGroup)
	, DynamicStateAttribute(InCollection, FGeometryDynamicCollection::DynamicStateAttribute,  FTransformCollection::TransformGroup)
	, InternalClusterParentTypeAttribute(InCollection, "InternalClusterParentTypeArray", FGeometryCollection::TransformGroup)
	, DynamicCollection(InCollection)
{
}

bool FGeometryCollectionDynamicStateFacade::IsValid() const
{
	return ActiveAttribute.IsValid()
		&& DynamicStateAttribute.IsValid()
		&& InternalClusterParentTypeAttribute.IsValid();
}

bool FGeometryCollectionDynamicStateFacade::IsActive(int32 TransformIndex) const
{
	return ActiveAttribute.Get()[TransformIndex];
}

bool FGeometryCollectionDynamicStateFacade::IsDynamicOrSleeping(int32 TransformIndex) const
{
	const int32 State = DynamicStateAttribute.Get()[TransformIndex];
	return (State == (int)EObjectStateTypeEnum::Chaos_Object_Sleeping) || (State == (int)EObjectStateTypeEnum::Chaos_Object_Dynamic);
}

bool FGeometryCollectionDynamicStateFacade::IsSleeping(int32 TransformIndex) const
{
	const int32 State = DynamicStateAttribute.Get()[TransformIndex];
	return (State == (int)EObjectStateTypeEnum::Chaos_Object_Sleeping);
}

bool FGeometryCollectionDynamicStateFacade::HasChildren(int32 TransformIndex) const
{
	return DynamicCollection.HasChildren(TransformIndex);
}

bool FGeometryCollectionDynamicStateFacade::HasBrokenOff(int32 TransformIndex) const
{
	const bool bIsActive = IsActive(TransformIndex);
	const bool bHasParent = DynamicCollection.GetHasParent(TransformIndex);
	return bIsActive && (!bHasParent) && IsDynamicOrSleeping(TransformIndex);
}

bool FGeometryCollectionDynamicStateFacade::HasInternalClusterParent(int32 TransformIndex) const
{
	const uint8 InternalParentType = InternalClusterParentTypeAttribute.Get()[TransformIndex];
	return InternalParentType != (uint8)Chaos::EInternalClusterType::None;
}

bool FGeometryCollectionDynamicStateFacade::HasDynamicInternalClusterParent(int32 TransformIndex) const
{
	const uint8 InternalParentType = InternalClusterParentTypeAttribute.Get()[TransformIndex];
	return InternalParentType == (uint8)Chaos::EInternalClusterType::Dynamic;
}

bool FGeometryCollectionDynamicStateFacade::HasClusterUnionParent(int32 TransformIndex) const
{
	const uint8 InternalParentType = InternalClusterParentTypeAttribute.Get()[TransformIndex];
	return InternalParentType == (uint8)Chaos::EInternalClusterType::ClusterUnion;
}