// Copyright Epic Games, Inc. All Rights Reserved.

#include "PBDRigidActiveParticlesBuffer.h"
#include "PBDRigidsSolver.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"

namespace Chaos
{

	FPBDRigidDirtyParticlesBuffer::FPBDRigidDirtyParticlesBuffer(const Chaos::EMultiBufferMode& InBufferMode, bool bInSingleThreaded) : BufferMode(InBufferMode), bUseLock(!bInSingleThreaded)
	{
		SolverDataOut = Chaos::FMultiBufferFactory<FPBDRigidDirtyParticlesBufferOut>::CreateBuffer(InBufferMode);
	}


	void FPBDRigidDirtyParticlesBuffer::CaptureSolverData(FPBDRigidsSolver* Solver)
	{
		SCOPE_CYCLE_COUNTER(STAT_CaptureSolverData);

		WriteLock();
		BufferPhysicsResults(Solver);
		FlipDataOut();
		WriteUnlock();
	}

	void FPBDRigidDirtyParticlesBuffer::BufferPhysicsResults(FPBDRigidsSolver* Solver)
	{
		auto& ActiveGameThreadParticles = SolverDataOut->AccessProducerBuffer()->DirtyGameThreadParticles;
		auto& PhysicsParticleProxies = SolverDataOut->AccessProducerBuffer()->PhysicsParticleProxies;

		ActiveGameThreadParticles.Empty();
		PhysicsParticleProxies.Empty();
		TParticleView<FPBDRigidParticles>& ActiveParticlesView = Solver->GetParticles().GetDirtyParticlesView();
		for (auto& ActiveParticle : ActiveParticlesView)
		{
			if (ActiveParticle.Handle())	//can this be null?
			{
				IPhysicsProxyBase* Proxy = ActiveParticle.Handle()->PhysicsProxy();
				
				if(Proxy != nullptr)	//can this be null?
				{
					if(Proxy->GetType() == EPhysicsProxyType::SingleParticleProxy)
					{
						ActiveGameThreadParticles.Add(static_cast<FSingleParticlePhysicsProxy*>(Proxy));
					}
					else
					{
						//must be a geometry collection
						PhysicsParticleProxies.Add(Proxy);
					}
				}	
			}
		}
	}

	void FPBDRigidDirtyParticlesBuffer::ReadLock()
	{
		if (bUseLock)
		{
			ResourceOutLock.ReadLock();
		}
	}

	void FPBDRigidDirtyParticlesBuffer::ReadUnlock()
	{
		if (bUseLock)
		{
			ResourceOutLock.ReadUnlock();
		}
	}

	void FPBDRigidDirtyParticlesBuffer::WriteLock()
	{
		if (bUseLock)
		{
			ResourceOutLock.WriteLock();
		}
	}

	void FPBDRigidDirtyParticlesBuffer::WriteUnlock()
	{
		if (bUseLock)
		{
			ResourceOutLock.WriteUnlock();
		}
	}
}
