// Copyright Epic Games, Inc. All Rights Reserved.

#include "PBDRigidsSolver.h"

#include "Async/AsyncWork.h"
#include "Chaos/ChaosArchive.h"
#include "Chaos/Character/CharacterGroundConstraint.h"
#include "Chaos/PBDCollisionConstraintsUtil.h"
#include "Chaos/Utilities.h"
#include "Chaos/ChaosDebugDraw.h"
#include "Chaos/PBDRigidsEvolution.h"
#include "ChaosStats.h"
#include "ChaosSolversModule.h"
#include "ChaosVisualDebugger/ChaosVisualDebuggerTrace.h"
#include "HAL/FileManager.h"
#include "Misc/ScopeLock.h"
#include "PhysicsProxy/CharacterGroundConstraintProxy.h"
#include "PhysicsProxy/ClusterUnionPhysicsProxy.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "PhysicsProxy/SkeletalMeshPhysicsProxy.h"
#include "PhysicsProxy/StaticMeshPhysicsProxy.h"
#include "PhysicsProxy/GeometryCollectionPhysicsProxy.h"
#include "PhysicsProxy/PerSolverFieldSystem.h"
#include "EventDefaults.h"
#include "EventsData.h"
#include "RewindData.h"
#include "ChaosSolverConfiguration.h"
#include "Chaos/PullPhysicsDataImp.h"
#include "Chaos/PhysicsSolverBaseImpl.h"
#include "Chaos/ConvexOptimizer.h"
#include "Chaos/AsyncInitBodyHelper.h"
#include "ChaosVisualDebugger/ChaosVDTraceMacros.h"

#include "Chaos/DebugDraw/DebugDrawParticle.h"
#include "ChaosDebugDraw/ChaosDDContext.h"
#include "ChaosDebugDraw/ChaosDDScene.h"
#include "ChaosDebugDraw/ChaosDDTimeline.h"

#include "ProfilingDebugging/CountersTrace.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "TaskDispatcherSolver.h"
#include "ChaosLog.h"

//PRAGMA_DISABLE_OPTIMIZATION

DEFINE_LOG_CATEGORY_STATIC(LogPBDRigidsSolver, Log, All);

// Stat Counters
DECLARE_DWORD_COUNTER_STAT(TEXT("NumDisabledBodies"), STAT_ChaosCounter_NumDisabledBodies, STATGROUP_ChaosCounters);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumBodies"), STAT_ChaosCounter_NumBodies, STATGROUP_ChaosCounters);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumDynamicBodies"), STAT_ChaosCounter_NumDynamicBodies, STATGROUP_ChaosCounters);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumKinematicBodies"), STAT_ChaosCounter_NumKinematicBodies, STATGROUP_ChaosCounters);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumStaticBodies"), STAT_ChaosCounter_NumStaticBodies, STATGROUP_ChaosCounters);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumGeomCollBodies"), STAT_ChaosCounter_NumGeometryCollectionBodies, STATGROUP_ChaosCounters);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumMovingBodies"), STAT_ChaosCounter_NumMovingBodies, STATGROUP_ChaosCounters);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumStaticShapes"), STAT_ChaosCounter_NumStaticShapes, STATGROUP_ChaosCounters);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumKinematicShapes"), STAT_ChaosCounter_NumKinematicShapes, STATGROUP_ChaosCounters);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumDynamicShapes"), STAT_ChaosCounter_NumDynamicShapes, STATGROUP_ChaosCounters);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumIslands"), STAT_ChaosCounter_NumIslands, STATGROUP_ChaosCounters);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumIslandGroups"), STAT_ChaosCounter_NumIslandGroups, STATGROUP_ChaosCounters);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumContacts"), STAT_ChaosCounter_NumContacts, STATGROUP_ChaosCounters);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumValidConstraints"), STAT_ChaosCounter_NumValidConstraints, STATGROUP_ChaosCounters);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumActiveConstraints"), STAT_ChaosCounter_NumActiveConstraints, STATGROUP_ChaosCounters);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumRestoredConstraints"), STAT_ChaosCounter_NumRestoredConstraints, STATGROUP_ChaosCounters);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumManifoldPoints"), STAT_ChaosCounter_NumManifoldPoints, STATGROUP_ChaosCounters);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumActiveManifoldPoints"), STAT_ChaosCounter_NumActiveManifoldPoints, STATGROUP_ChaosCounters);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumRestoredManifoldPoints"), STAT_ChaosCounter_NumRestoredManifoldPoints, STATGROUP_ChaosCounters);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumUpdatedManifoldPoints"), STAT_ChaosCounter_NumUpdatedManifoldPoints, STATGROUP_ChaosCounters);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumJoints"), STAT_ChaosCounter_NumJoints, STATGROUP_ChaosCounters);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumCharacterGroundConstraints"), STAT_ChaosCounter_NumCharacterGroundConstraints, STATGROUP_ChaosCounters);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumBroadPhasePairs"), STAT_ChaosCounter_NumBroadPhasePairs, STATGROUP_ChaosCounters);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumMidPhases"), STAT_ChaosCounter_NumMidPhases, STATGROUP_ChaosCounters);

TRACE_DECLARE_INT_COUNTER(ChaosTraceCounter_NumDisabledBodies, TEXT("Chaos/Solver/Bodies/NumDisabled"));
TRACE_DECLARE_INT_COUNTER(ChaosTraceCounter_NumBodies, TEXT("Chaos/Solver/Bodies/Num"));
TRACE_DECLARE_INT_COUNTER(ChaosTraceCounter_NumStaticBodies, TEXT("Chaos/Solver/Bodies/NumStatic"));
TRACE_DECLARE_INT_COUNTER(ChaosTraceCounter_NumKinematicBodies, TEXT("Chaos/Solver/Bodies/NumKinematic"));
TRACE_DECLARE_INT_COUNTER(ChaosTraceCounter_NumDynamicBodies, TEXT("Chaos/Solver/Bodies/NumDynamic"));
TRACE_DECLARE_INT_COUNTER(ChaosTraceCounter_NumGeometryCollectionBodies, TEXT("Chaos/Solver/Bodies/NumGC"));
TRACE_DECLARE_INT_COUNTER(ChaosTraceCounter_NumMovingBodies, TEXT("Chaos/Solver/Bodies/NumMoving"));
TRACE_DECLARE_INT_COUNTER(ChaosTraceCounter_NumStaticShapes, TEXT("Chaos/Solver/Collisions/NumStaticShapes"));
TRACE_DECLARE_INT_COUNTER(ChaosTraceCounter_NumKinematicShapes, TEXT("Chaos/Solver/Collisions/NumKinematicShapes"));
TRACE_DECLARE_INT_COUNTER(ChaosTraceCounter_NumDynamicShapes, TEXT("Chaos/Solver/Collisions/NumDynamicShapes"));
TRACE_DECLARE_INT_COUNTER(ChaosTraceCounter_NumIslands, TEXT("Chaos/Solver/Islands/NumIslands"));
TRACE_DECLARE_INT_COUNTER(ChaosTraceCounter_NumIslandGroups, TEXT("Chaos/Solver/Islands/NumIslandGroups"));
TRACE_DECLARE_INT_COUNTER(ChaosTraceCounter_NumContacts, TEXT("Chaos/Solver/Collisions/NumConstraints"));
TRACE_DECLARE_INT_COUNTER(ChaosTraceCounter_NumValidConstraints, TEXT("Chaos/Solver/Collisions/NumValidConstraints"));
TRACE_DECLARE_INT_COUNTER(ChaosTraceCounter_NumActiveConstraints, TEXT("Chaos/Solver/Collisions/NumActiveConstraints"));
TRACE_DECLARE_INT_COUNTER(ChaosTraceCounter_NumRestoredConstraints, TEXT("Chaos/Solver/Collisions/NumRestoredConstraints"));
TRACE_DECLARE_INT_COUNTER(ChaosTraceCounter_NumManifoldPoints, TEXT("Chaos/Solver/Collisions/NumManifoldPoints"));
TRACE_DECLARE_INT_COUNTER(ChaosTraceCounter_NumActiveManifoldPoints, TEXT("Chaos/Solver/Collisions/NumActiveManifoldPoints"));
TRACE_DECLARE_INT_COUNTER(ChaosTraceCounter_NumRestoredManifoldPoints, TEXT("Chaos/Solver/Collisions/NumRestoredManifoldPoints"));
TRACE_DECLARE_INT_COUNTER(ChaosTraceCounter_NumUpdatedManifoldPoints, TEXT("Chaos/Solver/Collisions/NumUpdatedManifoldPoints"));
TRACE_DECLARE_INT_COUNTER(ChaosTraceCounter_NumJoints, TEXT("Chaos/Solver/Joints/NumConstraints"));
TRACE_DECLARE_INT_COUNTER(ChaosTraceCounter_NumCharacterGroundConstraints, TEXT("Chaos/Solver/Character/NumConstraints"));
TRACE_DECLARE_INT_COUNTER(ChaosTraceCounter_NumBroadPhasePairs, TEXT("Chaos/Solver/Collisions/NumBroadPhasePairs"));
TRACE_DECLARE_INT_COUNTER(ChaosTraceCounter_NumMidPhases, TEXT("Chaos/Solver/Collisions/NumMidPhases"));

TRACE_DECLARE_INT_COUNTER(ChaosTraceCounter_MidPhase_NumShapePair, TEXT("Chaos/Solver/MidPhase/NumShapePair"));
TRACE_DECLARE_INT_COUNTER(ChaosTraceCounter_MidPhase_NumGeneric, TEXT("Chaos/Solver/MidPhase/NumGeneric"));

// Stat Iteration counters
DECLARE_DWORD_COUNTER_STAT(TEXT("NumPositionIterations"), STAT_ChaosCounter_NumPositionIterations, STATGROUP_ChaosCounters);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumVelocityIterations"), STAT_ChaosCounter_NumVelocityIterations, STATGROUP_ChaosCounters);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumProjectionIterations"), STAT_ChaosCounter_NumProjectionIterations, STATGROUP_ChaosCounters);

TRACE_DECLARE_INT_COUNTER(ChaosTraceCounter_NumPositionIterations, TEXT("Chaos/Solver/Iterations/NumPosition"));
TRACE_DECLARE_INT_COUNTER(ChaosTraceCounter_NumVelocityIterations, TEXT("Chaos/Solver/Iterations/NumVelocity"));
TRACE_DECLARE_INT_COUNTER(ChaosTraceCounter_NumProjectionIterations, TEXT("Chaos/Solver/Iterations/NumProjection"));

DECLARE_DWORD_COUNTER_STAT(TEXT("NumCollisionColors"), STAT_ChaosCounter_NumCollisionColors, STATGROUP_ChaosCounters);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumLinearJointColors"), STAT_ChaosCounter_NumLinearJointColors, STATGROUP_ChaosCounters);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumNonLinearJointColors"), STAT_ChaosCounter_NumNonLinearJointColors, STATGROUP_ChaosCounters);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumSuspensionConstraintColors"), STAT_ChaosCounter_NumSuspensionConstraintColors, STATGROUP_ChaosCounters);
DECLARE_DWORD_COUNTER_STAT(TEXT("NumCharacterGroundConstraintColors"), STAT_ChaosCounter_NumCharacterGroundConstraintColors, STATGROUP_ChaosCounters);

TRACE_DECLARE_INT_COUNTER(ChaosTraceCounter_NumCollisionColors, TEXT("Chaos/Solver/Colors/NumCollisionColors"));
TRACE_DECLARE_INT_COUNTER(ChaosTraceCounter_NumLinearJointColors, TEXT("Chaos/Solver/Colors/NumLinearJointColors"));
TRACE_DECLARE_INT_COUNTER(ChaosTraceCounter_NumNonLinearJointColors, TEXT("Chaos/Solver/Colors/NumNonLinearJointColors"));
TRACE_DECLARE_INT_COUNTER(ChaosTraceCounter_NumSuspensionConstraintColors, TEXT("Chaos/Solver/Colors/NumSuspensionConstraintColors"));
TRACE_DECLARE_INT_COUNTER(ChaosTraceCounter_NumCharacterGroundConstraintColors, TEXT("Chaos/Solver/Colors/NumCharacterGroundConstraintColors"));

// DebugDraw CVars
#if CHAOS_DEBUG_DRAW

// Must be 0 when checked in...
#define CHAOS_SOLVER_ENABLE_DEBUG_DRAW 0

namespace Chaos
{
	extern CHAOS_API bool bChaosDebugDraw_UseNewQueue;

	namespace DebugDraw
	{
		extern const FChaosDebugDrawColorsByState& GetDefaultShapesColorsPreIntegrate();
		extern const FChaosDebugDrawColorsByState& GetDefaultShapesColorsPostIntegrate();
		extern const FChaosDebugDrawColorsByState& GetDefaultShapesColorsCollisionDetection();
	}

	namespace CVars
	{
		int32 ChaosSolverDebugDrawShapes = CHAOS_SOLVER_ENABLE_DEBUG_DRAW;
		int32 ChaosSolverDebugDrawMass = 0;
		int32 ChaosSolverDebugDrawDensity = 0;
		int32 ChaosSolverDebugDrawBVHs = 0;
		int32 ChaosSolverDebugDrawCollisions = CHAOS_SOLVER_ENABLE_DEBUG_DRAW;
		int32 ChaosSolverDebugDrawCollidingShapes = 0;
		int32 ChaosSolverDebugDrawBounds = 0;
		int32 ChaosSolverDrawTransforms = 0;
		int32 ChaosSolverDrawIslands = 0;
		int32 ChaosSolverDebugDrawIslandSleepState = 0;
		int32 ChaosSolverDrawCCDInteractions = 0;
		int32 ChaosSolverDrawCCDThresholds = 0;
		int32 ChaosSolverDrawShapesShowStatic = 1;
		int32 ChaosSolverDrawShapesShowKinematic = 1;
		int32 ChaosSolverDrawShapesShowDynamic = 1;
		int32 ChaosSolverDrawJoints = 0;
		int32 ChaosSolverDrawCharacterGroundConstraints = 0;
		int32 ChaosSolverDebugDrawSpatialAccelerationStructure = 0;
		int32 ChaosSolverDebugDrawSpatialAccelerationStructureShowLeaves = 0;
		int32 ChaosSolverDebugDrawSpatialAccelerationStructureShowNodes = 0;
		int32 ChaosSolverDebugDrawSuspensionConstraints = 0;
		int32 ChaosSolverDrawClusterConstraints = 0;
		int32 ChaosSolverDebugDrawMeshContacts = 0;
		int32 ChaosSolverDebugDrawMeshContactDetails = 0;
		int32 ChaosSolverDebugDrawMeshBVHOverlaps = 0;
		int32 ChaosSolverDebugDrawColorShapeByClientServer = 0;
		int32 ChaosSolverDebugDrawShowServer = 1;
		int32 ChaosSolverDebugDrawShowClient = 1;
		DebugDraw::FChaosDebugDrawJointFeatures ChaosSolverDrawJointFeatures = DebugDraw::FChaosDebugDrawJointFeatures::MakeDefault();
		FAutoConsoleVariableRef CVarChaosSolverDrawShapes(TEXT("p.Chaos.Solver.DebugDrawShapes"), ChaosSolverDebugDrawShapes, TEXT("Draw Shapes (0 = never; 1 = end of frame)."));
		FAutoConsoleVariableRef CVarChaosSolverDebugDrawMass(TEXT("p.Chaos.Solver.DebugDrawMass"), ChaosSolverDebugDrawMass, TEXT("Draw Mass values in Kg (0 = never; 1 = end of frame)."));
		FAutoConsoleVariableRef CVarChaosSolverDebugDrawDensity(TEXT("p.Chaos.Solver.DebugDrawDensity"), ChaosSolverDebugDrawDensity, TEXT("Draw Density values in Kg/cm3 (0 = never; 1 = end of frame)."));
		FAutoConsoleVariableRef CVarChaosSolverDrawBVHs(TEXT("p.Chaos.Solver.DebugDrawBVHs"), ChaosSolverDebugDrawBVHs, TEXT("Draw Particle BVHs where applicable (0 = never; 1 = end of frame)."));
		FAutoConsoleVariableRef CVarChaosSolverDrawCollisions(TEXT("p.Chaos.Solver.DebugDrawCollisions"), ChaosSolverDebugDrawCollisions, TEXT("Draw Collisions (0 = never; 1 = end of frame)."));
		FAutoConsoleVariableRef CVarChaosSolverDrawCollidingShapes(TEXT("p.Chaos.Solver.DebugDrawCollidingShapes"), ChaosSolverDebugDrawCollidingShapes, TEXT("Draw Shapes that have collisions on them (0 = never; 1 = end of frame)."));
		FAutoConsoleVariableRef CVarChaosSolverDrawBounds(TEXT("p.Chaos.Solver.DebugDrawBounds"), ChaosSolverDebugDrawBounds, TEXT("Draw bounding volumes inside the broadphase (0 = never; 1 = end of frame)."));
		FAutoConsoleVariableRef CVarChaosSolverDrawTransforms(TEXT("p.Chaos.Solver.DebugDrawTransforms"), ChaosSolverDrawTransforms, TEXT("Draw particle transforms (0 = never; 1 = end of frame)."));
		FAutoConsoleVariableRef CVarChaosSolverDrawIslands(TEXT("p.Chaos.Solver.DebugDrawIslands"), ChaosSolverDrawIslands, TEXT("Draw solver islands (0 = never; 1 = end of frame)."));
		FAutoConsoleVariableRef CVarChaosSolverDrawIslandSleepState(TEXT("p.Chaos.Solver.DebugDrawSleepState"), ChaosSolverDebugDrawIslandSleepState, TEXT("Draw island sleep state."));
		FAutoConsoleVariableRef CVarChaosSolverDrawCCD(TEXT("p.Chaos.Solver.DebugDrawCCDInteractions"), ChaosSolverDrawCCDInteractions, TEXT("Draw CCD interactions."));
		FAutoConsoleVariableRef CVarChaosSolverDrawCCDThresholds(TEXT("p.Chaos.Solver.DebugDrawCCDThresholds"), ChaosSolverDrawCCDThresholds, TEXT("Draw CCD swept thresholds."));
		FAutoConsoleVariableRef CVarChaosSolverDrawShapesShapesStatic(TEXT("p.Chaos.Solver.DebugDraw.ShowStatics"), ChaosSolverDrawShapesShowStatic, TEXT("If DebugDrawShapes is enabled, whether to show static objects"));
		FAutoConsoleVariableRef CVarChaosSolverDrawShapesShapesKinematic(TEXT("p.Chaos.Solver.DebugDraw.ShowKinematics"), ChaosSolverDrawShapesShowKinematic, TEXT("If DebugDrawShapes is enabled, whether to show kinematic objects"));
		FAutoConsoleVariableRef CVarChaosSolverDrawShapesShapesDynamic(TEXT("p.Chaos.Solver.DebugDraw.ShowDynamics"), ChaosSolverDrawShapesShowDynamic, TEXT("If DebugDrawShapes is enabled, whether to show dynamic objects"));
		FAutoConsoleVariableRef CVarChaosSolverDrawJoints(TEXT("p.Chaos.Solver.DebugDrawJoints"), ChaosSolverDrawJoints, TEXT("Draw joints"));
		FAutoConsoleVariableRef CVarChaosSolverDrawCharacterGroundConstraints(TEXT("p.Chaos.Solver.DebugDrawCharacterGroundConstraints"), ChaosSolverDrawCharacterGroundConstraints, TEXT("Draw character ground constraints"));
		FAutoConsoleVariableRef CVarChaosSolverDebugDrawSpatialAccelerationStructure(TEXT("p.Chaos.Solver.DebugDrawSpatialAccelerationStructure"), ChaosSolverDebugDrawSpatialAccelerationStructure, TEXT("Draw spatial acceleration structure"));
		FAutoConsoleVariableRef CVarChaosSolverDebugDrawSpatialAccelerationStructureShowLeaves(TEXT("p.Chaos.Solver.DebugDrawSpatialAccelerationStructure.ShowLeaves"), ChaosSolverDebugDrawSpatialAccelerationStructureShowLeaves, TEXT("Show spatial acceleration structure leaves when its debug draw is enabled"));
		FAutoConsoleVariableRef CVarChaosSolverDebugDrawSpatialAccelerationStructureShowNodes(TEXT("p.Chaos.Solver.DebugDrawSpatialAccelerationStructure.ShowNodes"), ChaosSolverDebugDrawSpatialAccelerationStructureShowNodes, TEXT("Show spatial acceleration structure nodes when its debug draw is enabled"));
		FAutoConsoleVariableRef CVarChaosSolverDrawJointFeaturesCoMConnector(TEXT("p.Chaos.Solver.DebugDraw.JointFeatures.CoMConnector"), ChaosSolverDrawJointFeatures.bCoMConnector, TEXT("Joint features mask (see FDebugDrawJointFeatures)."));
		FAutoConsoleVariableRef CVarChaosSolverDrawJointFeaturesActorConnector(TEXT("p.Chaos.Solver.DebugDraw.JointFeatures.ActorConnector"), ChaosSolverDrawJointFeatures.bActorConnector, TEXT("Joint features mask (see FDebugDrawJointFeatures)."));
		FAutoConsoleVariableRef CVarChaosSolverDrawJointFeaturesStretch(TEXT("p.Chaos.Solver.DebugDraw.JointFeatures.Stretch"), ChaosSolverDrawJointFeatures.bStretch, TEXT("Joint features mask (see FDebugDrawJointFeatures)."));
		FAutoConsoleVariableRef CVarChaosSolverDrawJointFeaturesAxes(TEXT("p.Chaos.Solver.DebugDraw.JointFeatures.Axes"), ChaosSolverDrawJointFeatures.bAxes, TEXT("Joint features mask (see FDebugDrawJointFeatures)."));
		FAutoConsoleVariableRef CVarChaosSolverDrawJointFeaturesLevel(TEXT("p.Chaos.Solver.DebugDraw.JointFeatures.Level"), ChaosSolverDrawJointFeatures.bLevel, TEXT("Joint features mask (see FDebugDrawJointFeatures)."));
		FAutoConsoleVariableRef CVarChaosSolverDrawJointFeaturesIndex(TEXT("p.Chaos.Solver.DebugDraw.JointFeatures.Index"), ChaosSolverDrawJointFeatures.bIndex, TEXT("Joint features mask (see FDebugDrawJointFeatures)."));
		FAutoConsoleVariableRef CVarChaosSolverDrawJointFeaturesColor(TEXT("p.Chaos.Solver.DebugDraw.JointFeatures.Color"), ChaosSolverDrawJointFeatures.bColor, TEXT("Joint features mask (see FDebugDrawJointFeatures)."));
		FAutoConsoleVariableRef CVarChaosSolverDrawJointFeaturesIsland(TEXT("p.Chaos.Solver.DebugDraw.JointFeatures.Island"), ChaosSolverDrawJointFeatures.bIsland, TEXT("Joint features mask (see FDebugDrawJointFeatures)."));
		FAutoConsoleVariableRef CVarChaosSolverDrawSuspensionConstraints(TEXT("p.Chaos.Solver.DebugDrawSuspension"), ChaosSolverDebugDrawSuspensionConstraints, TEXT("Draw Suspension (0 = never; 1 = end of frame)."));
		FAutoConsoleVariableRef CVarChaosSolverDrawClusterConstraints(TEXT("p.Chaos.Solver.DebugDraw.Cluster.Constraints"), ChaosSolverDrawClusterConstraints, TEXT("Draw Active Cluster Constraints (0 = never; 1 = end of frame)."));
		FAutoConsoleVariableRef CVarChaosSolverDrawMeshContacts(TEXT("p.Chaos.Solver.DebugDrawMeshContacts"), ChaosSolverDebugDrawMeshContacts, TEXT("Draw Mesh contacts"));
		FAutoConsoleVariableRef CVarChaosSolverDrawMeshContactDetails(TEXT("p.Chaos.Solver.DebugDrawMeshContactDetails"), ChaosSolverDebugDrawMeshContactDetails, TEXT("Draw Mesh contacts"));
		FAutoConsoleVariableRef CVarChaosSolverDrawMeshBVHOverlaps(TEXT("p.Chaos.Solver.DebugDrawMeshBVHOverlaps"), ChaosSolverDebugDrawMeshBVHOverlaps, TEXT("Draw BVH of objects overlapping meshes"));
		FAutoConsoleVariableRef CVarChaosSolverDebugDrawColorShapeByClientServer(TEXT("p.Chaos.Solver.DebugDraw.ColorShapeByClientServer"), ChaosSolverDebugDrawColorShapeByClientServer, TEXT("Color shape according to client and server: red = server / blue = client "));
		FAutoConsoleVariableRef CVarChaosSolverDebugDrawShowServer(TEXT("p.Chaos.Solver.DebugDraw.ShowServer"), ChaosSolverDebugDrawShowServer, TEXT("Draw server related debug data"));
		FAutoConsoleVariableRef CVarChaosSolverDebugDrawShowClient(TEXT("p.Chaos.Solver.DebugDraw.ShowClient"), ChaosSolverDebugDrawShowClient, TEXT("Draw client related debug data"));

		int32 ChaosSolverDebugDrawPreIntegrationShapes = 0;
		int32 ChaosSolverDebugDrawPreIntegrationCollisions = 0;
		FAutoConsoleVariableRef CVarChaosSolverDrawPreIntegrationShapes(TEXT("p.Chaos.Solver.DebugDrawPreIntegrationShapes"), ChaosSolverDebugDrawPreIntegrationShapes, TEXT("Draw Shapes prior to integrate."));
		FAutoConsoleVariableRef CVarChaosSolverDrawPreIntegrationCollisions(TEXT("p.Chaos.Solver.DebugDrawPreIntegrationCollisions"), ChaosSolverDebugDrawPreIntegrationCollisions, TEXT("Draw Collisions prior to integrate."));

		int32 ChaosSolverDebugDrawPostIntegrationShapes = 0;
		int32 ChaosSolverDebugDrawPostIntegrationCollisions = 0;
		FAutoConsoleVariableRef CVarChaosSolverDrawPostIntegrationShapes(TEXT("p.Chaos.Solver.DebugDrawPostIntegrationShapes"), ChaosSolverDebugDrawPostIntegrationShapes, TEXT("Draw Shapes prior to constraint solve phase."));
		FAutoConsoleVariableRef CVarChaosSolverDrawPostIntegrationCollisions(TEXT("p.Chaos.Solver.DebugDrawPostIntegrationCollisions"), ChaosSolverDebugDrawPostIntegrationCollisions, TEXT("Draw Collisions prior to constraint solve phase."));


		DebugDraw::FChaosDebugDrawSettings ChaosSolverDebugDebugDrawSettings(
			/* ArrowSize =					*/ 10.0f,
			/* BodyAxisLen =				*/ 30.0f,
			/* ContactLen =					*/ 30.0f,
			/* ContactWidth =				*/ 6.0f,
			/* ContactInfoWidth				*/ 6.0f,
			/* ContactOwnerWidth =			*/ 0.0f,
			/* ConstraintAxisLen =			*/ 30.0f,
			/* JointComSize =				*/ 2.0f,
			/* LineThickness =				*/ 1.0f,
			/* DrawScale =					*/ 1.0f,
			/* FontHeight =					*/ 10.0f,
			/* FontScale =					*/ 1.5f,
			/* ShapeThicknesScale =			*/ 1.0f,
			/* PointSize =					*/ 5.0f,
			/* VelScale =					*/ 0.0f,
			/* AngVelScale =				*/ 0.0f,
			/* ImpulseScale =				*/ 0.0f,
			/* PushOutScale =				*/ 0.0f,
			/* InertiaScale =				*/ 1.0f,
			/* DrawPriority =				*/ 10,
			/* bShowSimple =				*/ true,
			/* bShowComplex =				*/ false,
			/* bInShowLevelSetCollision =	*/ true,
			/* InShapesColorsPerState =     */ DebugDraw::GetDefaultShapesColorsByState(),
			/* InShapesColorsPerShaepType=  */ DebugDraw::GetDefaultShapesColorsByShapeType(),
			/* InBoundsColorsPerState =     */ DebugDraw::GetDefaultBoundsColorsByState(),
			/* InBoundsColorsPerShapeType=  */ DebugDraw::GetDefaultBoundsColorsByShapeType()
		);

		extern DebugDraw::FChaosDebugDrawColorsByState GetSolverShapesColorsByState_Server();
		extern DebugDraw::FChaosDebugDrawColorsByState GetSolverShapesColorsByState_Client();

		DebugDraw::FChaosDebugDrawColorsByState GetSolverShapesColorsByState_Server()
		{
			static DebugDraw::FChaosDebugDrawColorsByState SolverShapesColorsByState_Server
			{
				/* InDynamicColor =	  */ FColor(255, 0, 0),
				/* InSleepingColor =  */ FColor(128, 0, 0),
				/* InKinematicColor = */ FColor(255, 0, 0),
				/* InStaticColor =	  */ FColor(255, 0, 0),
				/* InDebrisColor =	  */ FColor(255, 0, 0),
			};
			return SolverShapesColorsByState_Server;
		}

		DebugDraw::FChaosDebugDrawColorsByState GetSolverShapesColorsByState_Client()
		{
			static DebugDraw::FChaosDebugDrawColorsByState SolverShapesColorsByState_Client
			{
				/* InDynamicColor =	  */ FColor(0, 0, 255),
				/* InSleepingColor =  */ FColor(0, 0, 128),
				/* InKinematicColor = */ FColor(0, 0, 255),
				/* InStaticColor =	  */ FColor(0, 0, 255),
				/* InDebrisColor =	  */ FColor(0, 0, 255),
			};
			return SolverShapesColorsByState_Client;
		}

		FAutoConsoleVariableRef CVarChaosSolverArrowSize(TEXT("p.Chaos.Solver.DebugDraw.ArrowSize"), ChaosSolverDebugDebugDrawSettings.ArrowSize, TEXT("ArrowSize."));
		FAutoConsoleVariableRef CVarChaosSolverBodyAxisLen(TEXT("p.Chaos.Solver.DebugDraw.BodyAxisLen"), ChaosSolverDebugDebugDrawSettings.BodyAxisLen, TEXT("BodyAxisLen."));
		FAutoConsoleVariableRef CVarChaosSolverContactLen(TEXT("p.Chaos.Solver.DebugDraw.ContactLen"), ChaosSolverDebugDebugDrawSettings.ContactLen, TEXT("ContactLen."));
		FAutoConsoleVariableRef CVarChaosSolverContactWidth(TEXT("p.Chaos.Solver.DebugDraw.ContactWidth"), ChaosSolverDebugDebugDrawSettings.ContactWidth, TEXT("ContactWidth."));
		FAutoConsoleVariableRef CVarChaosSolverContactInfoWidth(TEXT("p.Chaos.Solver.DebugDraw.ContactInfoWidth"), ChaosSolverDebugDebugDrawSettings.ContactInfoWidth, TEXT("ContactInfoWidth."));
		FAutoConsoleVariableRef CVarChaosSolverContactOwnerWidth(TEXT("p.Chaos.Solver.DebugDraw.ContactOwnerWidth"), ChaosSolverDebugDebugDrawSettings.ContactOwnerWidth, TEXT("ContactOwnerWidth."));
		FAutoConsoleVariableRef CVarChaosSolverConstraintAxisLen(TEXT("p.Chaos.Solver.DebugDraw.ConstraintAxisLen"), ChaosSolverDebugDebugDrawSettings.ConstraintAxisLen, TEXT("ConstraintAxisLen."));
		FAutoConsoleVariableRef CVarChaosSolverLineThickness(TEXT("p.Chaos.Solver.DebugDraw.LineThickness"), ChaosSolverDebugDebugDrawSettings.LineThickness, TEXT("LineThickness."));
		FAutoConsoleVariableRef CVarChaosSolverLineShapeThickness(TEXT("p.Chaos.Solver.DebugDraw.ShapeLineThicknessScale"), ChaosSolverDebugDebugDrawSettings.ShapeThicknesScale, TEXT("Shape lineThickness multiplier."));
		FAutoConsoleVariableRef CVarChaosSolverPointSize(TEXT("p.Chaos.Solver.DebugDraw.PointSize"), ChaosSolverDebugDebugDrawSettings.PointSize, TEXT("Point size."));
		FAutoConsoleVariableRef CVarChaosSolverVelScale(TEXT("p.Chaos.Solver.DebugDraw.VelScale"), ChaosSolverDebugDebugDrawSettings.VelScale, TEXT("If >0 show velocity when drawing particle transforms."));
		FAutoConsoleVariableRef CVarChaosSolverAngVelScale(TEXT("p.Chaos.Solver.DebugDraw.AngVelScale"), ChaosSolverDebugDebugDrawSettings.AngVelScale, TEXT("If >0 show angular velocity when drawing particle transforms."));
		FAutoConsoleVariableRef CVarChaosSolverImpulseScale(TEXT("p.Chaos.Solver.DebugDraw.ImpulseScale"), ChaosSolverDebugDebugDrawSettings.ImpulseScale, TEXT("If >0 show impulses when drawing collisions."));
		FAutoConsoleVariableRef CVarChaosSolverPushOutScale(TEXT("p.Chaos.Solver.DebugDraw.PushOutScale"), ChaosSolverDebugDebugDrawSettings.PushOutScale, TEXT("If >0 show pushouts when drawing collisions."));
		FAutoConsoleVariableRef CVarChaosSolverInertiaScale(TEXT("p.Chaos.Solver.DebugDraw.InertiaScale"), ChaosSolverDebugDebugDrawSettings.InertiaScale, TEXT("When DebugDrawTransforms is enabled, show the mass-normalized inertia matrix scaled by this amount."));
		FAutoConsoleVariableRef CVarChaosSolverDrawPriority(TEXT("p.Chaos.Solver.DebugDraw.DrawPriority"), ChaosSolverDebugDebugDrawSettings.DrawPriority, TEXT("Draw Priority for debug draw shapes (0 draw at actual Z, +ve is closer to the screen)."));
		FAutoConsoleVariableRef CVarChaosSolverScale(TEXT("p.Chaos.Solver.DebugDraw.Scale"), ChaosSolverDebugDebugDrawSettings.DrawScale, TEXT("Scale applied to all Chaos Debug Draw line lengths etc."));
		FAutoConsoleVariableRef CVarChaosSolverShowSimple(TEXT("p.Chaos.Solver.DebugDraw.ShowSimple"), ChaosSolverDebugDebugDrawSettings.bShowSimpleCollision, TEXT("Whether to show simple collision is shape drawing is enabled"));
		FAutoConsoleVariableRef CVarChaosSolverShowComplex(TEXT("p.Chaos.Solver.DebugDraw.ShowComplex"), ChaosSolverDebugDebugDrawSettings.bShowComplexCollision, TEXT("Whether to show complex collision is shape drawing is enabled"));
		FAutoConsoleVariableRef CVarChaosSolverShowLevelSet(TEXT("p.Chaos.Solver.DebugDraw.ShowLevelSet"), ChaosSolverDebugDebugDrawSettings.bShowLevelSetCollision, TEXT("Whether to show levelset collision is shape drawing is enabled"));
	}
}
#endif

namespace Chaos
{
	namespace CVars
	{
		bool ChaosSolverUseParticlePool = true;
		FAutoConsoleVariableRef CVarChaosSolverUseParticlePool(TEXT("p.Chaos.Solver.UseParticlePool"), ChaosSolverUseParticlePool, TEXT("Whether or not to use dirty particle pool (Optim)"));

		int32 ChaosSolverParticlePoolNumFrameUntilShrink = 30;
		FAutoConsoleVariableRef CVarChaosSolverParticlePoolNumFrameUntilShrink(TEXT("p.Chaos.Solver.ParticlePoolNumFrameUntilShrink"), ChaosSolverParticlePoolNumFrameUntilShrink, TEXT("Num Frame until we can potentially shrink the pool"));

		// Enable/Disable collisions
		bool bChaosSolverCollisionEnabled = true;
		FAutoConsoleVariableRef CVarChaosSolverCollisionDisable(TEXT("p.Chaos.Solver.Collision.Enabled"), bChaosSolverCollisionEnabled, TEXT("Enable/Disable collisions in the main scene."));

		// Shrink particle arrays every frame to recove rmemory when a scene changes significantly
		bool bChaosSolverShrinkArrays = false;
		float ChaosArrayCollectionMaxSlackFraction = 0.5f;
		int32 ChaosArrayCollectionMinSlack = 100;
		FAutoConsoleVariableRef CVarChaosSolverShrinkArrays(TEXT("p.Chaos.Solver.ShrinkArrays"), bChaosSolverShrinkArrays, TEXT("Enable/Disable particle array shrinking in the main scene"));
		FAutoConsoleVariableRef CVarChaosSolverArrayCollectionMaxSlackFraction(TEXT("p.Chaos.ArrayCollection.MaxSlackFraction"), ChaosArrayCollectionMaxSlackFraction, TEXT("Shrink particle arrays if the number of slack elements exceeds the number of elements by this fraction"));
		FAutoConsoleVariableRef CVarChaosSolverArrayCollectionMinSlack(TEXT("p.Chaos.ArrayCollection.MinSlack"), ChaosArrayCollectionMinSlack, TEXT("Do not reduce the size of particle arrays if it would leave less slack than this"));

		// Iteration count cvars
		// These override the engine config if >= 0

		int32 ChaosSolverCollisionPositionFrictionIterations = -1;
		int32 ChaosSolverCollisionVelocityFrictionIterations = -1;
		int32 ChaosSolverCollisionPositionShockPropagationIterations = -1;
		int32 ChaosSolverCollisionVelocityShockPropagationIterations = -1;
		FAutoConsoleVariableRef CVarChaosSolverCollisionPositionFrictionIterations(TEXT("p.Chaos.Solver.Collision.PositionFrictionIterations"), ChaosSolverCollisionPositionFrictionIterations, TEXT("Override number of position iterations where friction is applied (if >= 0)"));
		FAutoConsoleVariableRef CVarChaosSolverCollisionVelocityFrictionIterations(TEXT("p.Chaos.Solver.Collision.VelocityFrictionIterations"), ChaosSolverCollisionVelocityFrictionIterations, TEXT("Override number of velocity iterations where friction is applied (if >= 0)"));
		FAutoConsoleVariableRef CVarChaosSolverCollisionPositionShockPropagationIterations(TEXT("p.Chaos.Solver.Collision.PositionShockPropagationIterations"), ChaosSolverCollisionPositionShockPropagationIterations, TEXT("Override number of position iterations where shock propagation is applied (if >= 0)"));
		FAutoConsoleVariableRef CVarChaosSolverCollisionVelocityShockPropagationIterations(TEXT("p.Chaos.Solver.Collision.VelocityShockPropagationIterations"), ChaosSolverCollisionVelocityShockPropagationIterations, TEXT("Override number of velocity iterations where shock propagation is applied (if >= 0)"));

		int32 ChaosSolverPositionIterations = -1;
		FAutoConsoleVariableRef CVarChaosSolverIterations(TEXT("p.Chaos.Solver.Iterations.Position"), ChaosSolverPositionIterations, TEXT("Override number of solver position iterations (-1 to use config)"));

		int32 ChaosSolverVelocityIterations = -1;
		FAutoConsoleVariableRef CVarChaosSolverPushOutIterations(TEXT("p.Chaos.Solver.Iterations.Velocity"), ChaosSolverVelocityIterations, TEXT("Override number of solver velocity iterations (-1 to use config)"));

		int32 ChaosSolverProjectionIterations = -1;
		FAutoConsoleVariableRef CVarChaosSolverProjectionIterations(TEXT("p.Chaos.Solver.Iterations.Projection"), ChaosSolverProjectionIterations, TEXT("Override number of solver projection iterations (-1 to use config)"));

		int32 ChaosSolverDeterministic = -1;
		FAutoConsoleVariableRef CVarChaosSolverDeterministic(TEXT("p.Chaos.Solver.Deterministic"), ChaosSolverDeterministic, TEXT("Override determinism. 0: disabled; 1: enabled; -1: use config"));

		// Whether to force split-impulse (momentumless position correction) mode in the constraint solver. -1 = no overide; 0 = force off; 1 = force on
		// NOTE: only supported by collision solver (not joints etc)
		int32 ChaosSolverSplitImpulseMode = -1;
		FAutoConsoleVariableRef CVarChaosCollisionSplitImpulseMode(TEXT("p.Chaos.Solver.SplitImpulseMode"), ChaosSolverSplitImpulseMode, TEXT("EXPERIMENTAL. Whether to force split-impulse mode. Currently implemented for collison only."));

		// Copied from RBAN
		FRealSingle ChaosSolverJointPositionTolerance = 0.025f;
		FAutoConsoleVariableRef CVarChaosSolverJointPositionTolerance(TEXT("p.Chaos.Solver.Joint.PositionTolerance"), ChaosSolverJointPositionTolerance, TEXT("PositionTolerance."));
		FRealSingle ChaosSolverJointAngleTolerance = 0.001f;
		FAutoConsoleVariableRef CVarChaosSolverJointAngleTolerance(TEXT("p.Chaos.Solver.Joint.AngleTolerance"), ChaosSolverJointAngleTolerance, TEXT("AngleTolerance."));
		FRealSingle ChaosSolverJointMinParentMassRatio = 0.2f;
		FAutoConsoleVariableRef CVarChaosSolverJointMinParentMassRatio(TEXT("p.Chaos.Solver.Joint.MinParentMassRatio"), ChaosSolverJointMinParentMassRatio, TEXT("6Dof joint MinParentMassRatio (if > 0)"));
		FRealSingle ChaosSolverJointMaxInertiaRatio = 5.0f;
		FAutoConsoleVariableRef CVarChaosSolverJointMaxInertiaRatio(TEXT("p.Chaos.Solver.Joint.MaxInertiaRatio"), ChaosSolverJointMaxInertiaRatio, TEXT("6Dof joint MaxInertiaRatio (if > 0)"));

		// Collision detection cvars

		// Utility to support runtime changes to some high-level collision configuration that requires we update all existing collisions (actually we just destroy them)
		// This is only intended for testing and debugging and handling the change is not fast.
		bool bChaosCollisionConfigChanged = false;
		FConsoleVariableDelegate OnCollisionConfigCVarChanged = FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* CVar) -> void { bChaosCollisionConfigChanged = true; });

		// These override the engine config if >= 0
		FRealSingle ChaosSolverCullDistance = -1.0f;
		FAutoConsoleVariableRef CVarChaosSolverCullDistance(TEXT("p.Chaos.Solver.Collision.CullDistance"), ChaosSolverCullDistance, TEXT("Override cull distance (if >= 0)"), OnCollisionConfigCVarChanged);

		// @todo(chaos): move to physics project settings and set these to -1 when we are settled on values...
		FRealSingle ChaosSolverVelocityBoundsMultiplier = 1.0f;
		FRealSingle ChaosSolverMaxVelocityBoundsExpansion = 3.0f;			// This should probably be a fraction of object size (see FParticlePairMidPhase::GenerateCollisions)
		FRealSingle ChaosSolverVelocityBoundsMultiplierMACD = 1.0f;
		FRealSingle ChaosSolverMaxVelocityBoundsExpansionMACD = 1000.0f;	// For use when Movement-Aware Collision Detection (MACD) is enabled
		FAutoConsoleVariableRef CVarChaosSolverVelocityBoundsMultiplier(TEXT("p.Chaos.Solver.Collision.VelocityBoundsMultiplier"), ChaosSolverVelocityBoundsMultiplier, TEXT("Override velocity bounds multiplier (if >= 0)"), OnCollisionConfigCVarChanged);
		FAutoConsoleVariableRef CVarChaosSolverMaxVelocityBoundsExpansion(TEXT("p.Chaos.Solver.Collision.MaxVelocityBoundsExpansion"), ChaosSolverMaxVelocityBoundsExpansion, TEXT("Override max velocity bounds expansion (if >= 0)"), OnCollisionConfigCVarChanged);
		FAutoConsoleVariableRef CVarChaosSolverVelocityBoundsMultiplierMACD(TEXT("p.Chaos.Solver.Collision.VelocityBoundsMultiplierMACD"), ChaosSolverVelocityBoundsMultiplierMACD, TEXT("Override velocity bounds multiplier for MACD (if >= 0)"), OnCollisionConfigCVarChanged);
		FAutoConsoleVariableRef CVarChaosSolverMaxVelocityBoundsExpansionMACD(TEXT("p.Chaos.Solver.Collision.MaxVelocityBoundsExpansionMACD"), ChaosSolverMaxVelocityBoundsExpansionMACD, TEXT("Override max velocity bounds expansion for MACD (if >= 0)"), OnCollisionConfigCVarChanged);

		FRealSingle ChaosSolverMaxPushOutVelocity = -1.0f;
		FAutoConsoleVariableRef CVarChaosSolverMaxPushOutVelocity(TEXT("p.Chaos.Solver.Collision.MaxPushOutVelocity"), ChaosSolverMaxPushOutVelocity, TEXT("Override max pushout velocity (if >= 0)"), OnCollisionConfigCVarChanged);

		FRealSingle ChaosSolverDepenetrationVelocity = -1.0f;
		FAutoConsoleVariableRef CVarChaosSolverInitialOverlapDepentrationVelocity(TEXT("p.Chaos.Solver.Collision.DepenetrationVelocity"), ChaosSolverDepenetrationVelocity, TEXT("Override initial overlap depenetration velocity (if >= 0)"), OnCollisionConfigCVarChanged);

		int32 ChaosSolverCleanupCommandsOnDestruction = 1;
		FAutoConsoleVariableRef CVarChaosSolverCleanupCommandsOnDestruction(TEXT("p.Chaos.Solver.CleanupCommandsOnDestruction"), ChaosSolverCleanupCommandsOnDestruction, TEXT("Whether or not to run internal command queue cleanup on solver destruction (0 = no cleanup, >0 = cleanup all commands)"));

		int32 ChaosSolverCollisionDeferNarrowPhase = 0;
		FAutoConsoleVariableRef CVarChaosSolverCollisionDeferNarrowPhase(TEXT("p.Chaos.Solver.Collision.DeferNarrowPhase"), ChaosSolverCollisionDeferNarrowPhase, TEXT("Create contacts for all broadphase pairs, perform NarrowPhase later."));

		// Allow one-shot or incremental manifolds where supported (which depends on shape pair types)
		int32 ChaosSolverCollisionUseManifolds = 1;
		FAutoConsoleVariableRef CVarChaosSolverCollisionUseManifolds(TEXT("p.Chaos.Solver.Collision.UseManifolds"), ChaosSolverCollisionUseManifolds, TEXT("Enable/Disable use of manifolds in collision."));

		// Allow manifolds to be reused between ticks
		int32 ChaosSolverCollisionAllowManifoldUpdate = 1;
		FAutoConsoleVariableRef CVarChaosSolverCollisionAllowManifoldUpdate(TEXT("p.Chaos.Solver.Collision.AllowManifoldUpdate"), ChaosSolverCollisionAllowManifoldUpdate, TEXT("Enable/Disable reuse of manifolds between ticks (for small movement)."));

		// Enable/Disable CCD. Set to false to disable the system, regardless of particle settings
		bool bChaosUseCCD = true;
		FAutoConsoleVariableRef  CVarChaosUseCCD(TEXT("p.Chaos.Solver.UseCCD"), bChaosUseCCD, TEXT("Global flag to turn CCD on or off. Default is true (on)"), OnCollisionConfigCVarChanged);

		// Enable/Disable MACD (Motion-Aware Collision Detection). Set to false to disable the system, regardless of particle settings
		bool bChaosUseMACD = true;
		FAutoConsoleVariableRef CVarChaos_Collision_UseMACD(TEXT("p.Chaos.Solver.UseMACD"), bChaosUseMACD, TEXT("Global flag to turn Movement-Aware Collision Detection (MACD) on or off. Default is true (on)"), OnCollisionConfigCVarChanged);

		// Use to force all collisions to use MACD for testing (must also have bChaosUseMACD enabled)
		bool bChaosForceMACD = false;
		FAutoConsoleVariableRef CVarChaos_Collision_ForceMACD(TEXT("p.Chaos.Solver.bChaosForceMACD"), bChaosForceMACD, TEXT("Force all collisions to use MACD for testing"), OnCollisionConfigCVarChanged);

		// Joint cvars
		float ChaosSolverJointMinSolverStiffness = 1.0f;
		float ChaosSolverJointMaxSolverStiffness = 1.0f;
		int32 ChaosSolverJointNumIterationsAtMaxSolverStiffness = 1;
		bool bChaosSolverJointSolvePositionLast = true;
		bool bChaosSolverJointUsePositionBasedDrives = true;
		int32 ChaosSolverJointNumShockProagationIterations = 0;
		FRealSingle ChaosSolverJointShockPropagation = -1.0f;
		FAutoConsoleVariableRef CVarChaosSolverJointMinSolverStiffness(TEXT("p.Chaos.Solver.Joint.MinSolverStiffness"), ChaosSolverJointMinSolverStiffness, TEXT("Solver stiffness on first iteration, increases each iteration toward MaxSolverStiffness."));
		FAutoConsoleVariableRef CVarChaosSolverJointMaxSolverStiffness(TEXT("p.Chaos.Solver.Joint.MaxSolverStiffness"), ChaosSolverJointMaxSolverStiffness, TEXT("Solver stiffness on last iteration, increases each iteration from MinSolverStiffness."));
		FAutoConsoleVariableRef CVarChaosSolverJointNumIterationsAtMaxSolverStiffness(TEXT("p.Chaos.Solver.Joint.NumIterationsAtMaxSolverStiffness"), ChaosSolverJointNumIterationsAtMaxSolverStiffness, TEXT("How many iterations we want at MaxSolverStiffness."));
		FAutoConsoleVariableRef CVarChaosSolverJointSolvePositionFirst(TEXT("p.Chaos.Solver.Joint.SolvePositionLast"), bChaosSolverJointSolvePositionLast, TEXT("Should we solve joints in position-then-rotation order (false) or rotation-then-position order (true, default)"));
		FAutoConsoleVariableRef CVarChaosSolverJointUsePBDVelocityDrives(TEXT("p.Chaos.Solver.Joint.UsePBDDrives"), bChaosSolverJointUsePositionBasedDrives, TEXT("Whether to solve drives in the position or velocity phase of the solver (default true"));
		FAutoConsoleVariableRef CVarChaosSolverJointNumShockPropagationIterations(TEXT("p.Chaos.Solver.Joint.NumShockPropagationIterations"), ChaosSolverJointNumShockProagationIterations, TEXT("How many iterations to enable SHockProagation for."));
		FAutoConsoleVariableRef CVarChaosSolverJointShockPropagation(TEXT("p.Chaos.Solver.Joint.ShockPropagation"), ChaosSolverJointShockPropagation, TEXT("6Dof joint shock propagation override (if >= 0)."));

		int32 ChaosVisualDebuggerEnable = 1;
		FAutoConsoleVariableRef CVarChaosVisualDebuggerEnable(TEXT("p.Chaos.VisualDebuggerEnable"), ChaosVisualDebuggerEnable, TEXT("Enable/Disable pushing/saving data to the visual debugger"));

		bool bChaosEnableOverrideSingleParticleTypeToCluster = false;
		FAutoConsoleVariableRef CVarChaosEnableOverrideSingleParticleTypeToCluster(TEXT("p.Chaos.EnableOverrideSingleParticleTypeToCluster"), bChaosEnableOverrideSingleParticleTypeToCluster, TEXT("When enabled particles are promoted to cluster type so they may be used with cluster unions"));

		// Enable a couple bug fixes with temporary roll-back just in case
		bool bRemoveParticleFromMovingKinematicsOnDisable = true;
		FAutoConsoleVariableRef CVarChaosRemoveParticleFromMovingKinematicsOnDisable(TEXT("p.Chaos.RemoveParticleFromMovingKinematicsOnDisable"), bRemoveParticleFromMovingKinematicsOnDisable, TEXT(""));

		bool bSingleThreadPullData = false;
		FAutoConsoleVariableRef CVarChaosSingleThreadPushData(TEXT("p.Chaos.SingleThreadPullData"), bSingleThreadPullData, TEXT("Run pull Data in Single Thread."));
	}
}

namespace PhysicsReplicationCVars
{
	namespace ResimulationCVars
	{
		bool bApplyTargetsWhileResimulating = true;
		static FAutoConsoleVariableRef CVarResimApplyTargetsWhileResimulating(TEXT("np2.Resim.ApplyTargetsWhileResimulating"), bApplyTargetsWhileResimulating, TEXT("If false, target states from the server are only applied on rewind. If true, target states from the server are applied during resimulation if there are any available."));

		bool bEnableAdaptiveCoalescing = false;
		static FAutoConsoleVariableRef CVarResimEnableAdaptiveCoalescing(TEXT("p.Resim.EnableAdaptiveCoalescing"), bEnableAdaptiveCoalescing, TEXT("When enabled, resimulation adaptively groups frames and simulates with larger delta time to stay within a time budget. Last resort optimization to prevent game thread stalls from long resimulations."));

		float CoalescingBudgetMultiplier = 1.0f;
		static FAutoConsoleVariableRef CVarResimCoalescingBudgetMultiplier(TEXT("p.Resim.CoalescingBudgetMultiplier"), CoalescingBudgetMultiplier, TEXT("Multiplier on the fixed physics dt that defines the resim time budget. Coalescing activates when projected resim time exceeds FixedDt * this value."));

		int32 MaxCoalesceMultiplier = 3;
		static FAutoConsoleVariableRef CVarResimMaxCoalesceMultiplier(TEXT("p.Resim.MaxCoalesceMultiplier"), MaxCoalesceMultiplier, TEXT("Maximum frame grouping multiplier for adaptive coalescing (2 = max 2x dt, 3 = max 3x dt)."));

		int32 AccurateHeadFrames = 3;
		static FAutoConsoleVariableRef CVarResimAccurateHeadFrames(TEXT("p.Resim.AccurateHeadFrames"), AccurateHeadFrames, TEXT("Number of frames at the start of resimulation that always run at full fidelity (1x dt) when adaptive coalescing is active. Also serves as the minimum sample count for the per-step cost estimate before coalescing can activate."));
	}
}

namespace Chaos
{
	using namespace CVars;

	CHAOS_API extern int32 SyncKinematicOnGameThread;

	class AdvanceOneTimeStepTask : public FNonAbandonableTask
	{
		friend class FAutoDeleteAsyncTask<AdvanceOneTimeStepTask>;
	public:
		AdvanceOneTimeStepTask(
			FPBDRigidsSolver* Scene
			, const FReal DeltaTime
			, const FSubStepInfo& SubStepInfo)
			: MSolver(Scene)
			, MDeltaTime(DeltaTime)
			, MSubStepInfo(SubStepInfo)
		{
			UE_LOGF(LogPBDRigidsSolver, Verbose, "AdvanceOneTimeStepTask::AdvanceOneTimeStepTask()");
		}

		void DoWork()
		{
			LLM_SCOPE(ELLMTag::ChaosUpdate);
			UE_LOGF(LogPBDRigidsSolver, Verbose, "AdvanceOneTimeStepTask::DoWork()");

			MSolver->ResetStatCounters();

			if (FRewindData* RewindData = MSolver->GetRewindData())
			{
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				RewindData->ApplyInputs(MSolver->GetCurrentFrame(), MSolver->GetEvolution()->IsResetting());
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}

			{
				QUICK_SCOPE_CYCLE_COUNTER(Chaos_ApplyCallbacks_Internal);
				MSolver->ApplyCallbacks_Internal();
			}
			
			{
				SCOPE_CYCLE_COUNTER(STAT_UpdateParams);
				FPBDPositionConstraints PositionTarget; // Dummy for now
				TMap<int32, int32> TargetedParticles;
				{
					MSolver->FieldParameterUpdateCallback(PositionTarget, TargetedParticles);
				}

				for (FGeometryCollectionPhysicsProxy* GeoclObj : MSolver->GetGeometryCollectionPhysicsProxiesField_Internal())
				{
					GeoclObj->FieldParameterUpdateCallback(MSolver);
				}

				MSolver->GetEvolution()->GetBroadPhase().GetIgnoreCollisionManager().ProcessPendingQueues(*MSolver);
			}

			{
				//SCOPE_CYCLE_COUNTER(STAT_BeginFrame);
				//MSolver->StartFrameCallback(MDeltaTime, MSolver->GetSolverTime());
			}


			if(FRewindData* RewindData = MSolver->GetRewindData())
			{
				QUICK_SCOPE_CYCLE_COUNTER(Chaos_RewindData_AdvanceFrame);
				RewindData->AdvanceFrame(MDeltaTime,[Evolution = MSolver->GetEvolution()]()
				{
					return Evolution->CreateExternalResimCache();
				});
			}

			{
				SCOPE_CYCLE_COUNTER(STAT_EvolutionAndKinematicUpdate);

				// This needs to be before BeginFrame since there's a possibility that we'll need to invalidate particles when updating cluster unions.
				// Invalidating particles will cause us to iterate through and remove active collisions. If this is after BeginFrame but before we run
				// the next broadphase, we'll pass the epoch check but have an invalid index into a now-empty active constraints array.
				MSolver->GetEvolution()->GetRigidClustering().UnionClusterGroups();

				// Process any sleep/wake requests that came from the game thread
				// NOTE: Must be before GetCollisionConstraints().BeginFrame() (because its behaviour depends on sleep state)
				MSolver->GetEvolution()->GetIslandManager().UpdateExplicitSleep();

				// clear out the collision constraints as they will be stale from last frame if AdvanceOneTimeStep never gets called due to TimeRemaining being less than MinDeltaTime 
				// @todo(chaos): maybe we can pull data at a better time instead to avoid collision-specific code here for event dispatch
				MSolver->GetEvolution()->GetCollisionConstraints().BeginFrame();
			
				// This outer loop can potentially cause the system to lose energy over integration
				// in a couple of different cases.
				//
				// * If we have a timestep that's smaller than MinDeltaTime, then we just won't step.
				//   Yes, we'll lose some teeny amount of energy, but we'll avoid 1/dt issues.
				//
				// * If we have used all of our substeps but still have time remaining, then some
				//   energy will be lost.
				const FReal MinDeltaTime = MSolver->GetMinDeltaTime_External();
				const FReal MaxDeltaTime = MSolver->GetMaxDeltaTime_External();
				int32 StepsRemaining = MSubStepInfo.bSolverSubstepped ? 1 : MSolver->GetMaxSubSteps_External();
				FReal TimeRemaining = MDeltaTime;
				bool bFirstStep = true;
				while (StepsRemaining > 0 && TimeRemaining > MinDeltaTime)
				{
					--StepsRemaining;
					const FReal DeltaTime = MaxDeltaTime > 0.f ? FMath::Min(TimeRemaining, MaxDeltaTime) : TimeRemaining;
					TimeRemaining -= DeltaTime;

					{
						MSolver->FieldForcesUpdateCallback();
					}

					for (FGeometryCollectionPhysicsProxy* GeoCollectionObj : MSolver->GetGeometryCollectionPhysicsProxiesField_Internal())
					{
						GeoCollectionObj->FieldForcesUpdateCallback(MSolver);
					}
					// we only want to process once per step because we don't want to apply the same forces multiple time during this solver tick
					MSolver->GetGeometryCollectionPhysicsProxiesField_Internal().Reset();

					if(FRewindData* RewindData = MSolver->GetRewindData())
					{
						//todo: make this work with sub-stepping
						MSolver->GetEvolution()->SetCurrentStepResimCache(bFirstStep ? RewindData->GetCurrentStepResimCache() : nullptr);
					}

					MSolver->GetEvolution()->AdvanceOneTimeStep(DeltaTime, MSubStepInfo);
					bFirstStep = false;
				}

				// Editor will tick with 0 DT, this will guarantee acceleration structure is still processing even if we don't advance evolution.
				if (MDeltaTime < MinDeltaTime)
				{
					MSolver->GetEvolution()->ComputeIntermediateSpatialAcceleration();
				}


#if CHAOS_CHECKED
				// If time remains, then log why we have lost energy over the timestep.
				if (TimeRemaining > 0.f)
				{
					if (StepsRemaining == 0)
					{
						UE_LOGF(LogPBDRigidsSolver, Warning, "AdvanceOneTimeStepTask::DoWork() - Energy lost over %fs due to too many substeps over large timestep", TimeRemaining);
					}
					else
					{
						UE_LOGF(LogPBDRigidsSolver, Warning, "AdvanceOneTimeStepTask::DoWork() - Energy lost over %fs due to small timestep remainder", TimeRemaining);
					}
				}
#endif
			}

			{
				SCOPE_CYCLE_COUNTER(STAT_EventDataGathering);
				{
					SCOPE_CYCLE_COUNTER(STAT_FillProducerData);
					// The Game Thread is now in charge to reset the producer buffer
					constexpr bool bResetProducerData = false;
					MSolver->GetEventManager()->FillProducerData(MSolver, bResetProducerData);
					MSolver->GetEvolution()->ResetAllRemovals();
				}
			}
			
			{
				SCOPE_CYCLE_COUNTER(STAT_EndFrame);
				MSolver->GetEvolution()->EndFrame(MDeltaTime);
			}

			{
				SCOPE_CYCLE_COUNTER(STAT_FinalizeCallbacks);
				MSolver->FinalizeCallbackData_Internal();
			}

			MSolver->SetSolverTime(MSolver->GetSolverTime() + MDeltaTime );
			MSolver->GetCurrentFrame()++;
			MSolver->PostTickDebugDraw(MDeltaTime);

			MSolver->UpdateStatCounters();

			//Editor ticks with 0 dt. We don't want to buffer any dirty data from this since it won't be consumed
			//TODO: handle this more gracefully
			if(MDeltaTime > 0)
			{
				CSV_SCOPED_TIMING_STAT(PhysicsVerbose, StepSolver_PullData);
				MSolver->CompleteSceneSimulation();
			}

			{
				SCOPE_CYCLE_COUNTER(STAT_ResetClusteringEvents);

				// reset all clustering events needs to be after CompleteSceneSimulation to make sure the cache recording gets them before they get removed
				// they cannot be right after the presolve callback ( as they were before ) because they will cause geometry collection replicated clients to miss them
				// Todo(chaos) we should probably move all of the solver event reset here in the future
				MSolver->GetEvolution()->GetRigidClustering().ResetAllEvents();
			}

			// Recover unused memory from particle arrays, based on the array shrink policy
			if (bChaosSolverShrinkArrays)
			{
				QUICK_SCOPE_CYCLE_COUNTER(ShrinkParticleArrays);
				MSolver->GetParticles().ShrinkArrays(CVars::ChaosArrayCollectionMaxSlackFraction, CVars::ChaosArrayCollectionMinSlack);
			}

			if (FRewindData* RewindData = MSolver->GetRewindData())
			{
				SCOPE_CYCLE_COUNTER(STAT_RewindFinishFrame);
				RewindData->FinishFrame();
			}
		}

	protected:

		TStatId GetStatId() const
		{
			RETURN_QUICK_DECLARE_CYCLE_STAT(AdvanceOneTimeStepTask, STATGROUP_ThreadPoolAsyncTasks);
		}

		FPBDRigidsSolver* MSolver;
		FReal MDeltaTime;
		FSubStepInfo MSubStepInfo;
		TSharedPtr<FCriticalSection> PrevLock, CurrentLock;
		TSharedPtr<FEvent> PrevEvent, CurrentEvent;
	};

	FPBDRigidsSolver::FPBDRigidsSolver(const EMultiBufferMode BufferingModeIn, UObject* InOwner, FReal InAsyncDt)
		: Super(BufferingModeIn, BufferingModeIn == EMultiBufferMode::Single ? EThreadingModeTemp::SingleThread : EThreadingModeTemp::TaskGraph, InOwner, InAsyncDt)
		, PhysSceneHack(nullptr)
		, CurrentFrame(0)
		, bHasFloor(true)
		, bIsFloorAnalytic(false)
		, FloorHeight(0.f)
		, bIsDeterministic(false)
		, TaskDispatcher(MakeUnique<Private::FTaskDispatcherSolver>())
		, QueryMaterials_External()
		, QueryMaterialMasks_External()
		, SimMaterials()
		, SimMaterialMasks()
		, UniqueIndices()
		, Particles(UniqueIndices)
		, MEvolution(new FPBDRigidsEvolution(Particles, SimMaterials, &MidPhaseModifiers, &CCDModifiers, &StrainModifiers, &ContactModifiers, BufferingModeIn == EMultiBufferMode::Single))
		, MEventManager(new FEventManager(BufferingModeIn))
		, MSolverEventFilters(new FSolverEventFilters())
		, MDirtyParticlesBuffer(new FDirtyParticlesBuffer(BufferingModeIn, BufferingModeIn == EMultiBufferMode::Single))
		, MCurrentLock(new FCriticalSection())

		, PerSolverField(nullptr)
		, Serializer(this)
	{
		UE_LOGF(LogPBDRigidsSolver, Verbose, "PBDRigidsSolver::PBDRigidsSolver()");
		Reset();

		MEvolution->SetInternalParticleInitilizationFunction(
			[this](const FGeometryParticleHandle* OldParticle, FGeometryParticleHandle* NewParticle) 
			{
				IPhysicsProxyBase* Proxy = const_cast<IPhysicsProxyBase*>(OldParticle->PhysicsProxy());
				if (FPBDRigidClusteredParticleHandle* NewClusteredParticle = NewParticle->CastToClustered())
				{
					NewClusteredParticle->AddPhysicsProxy(Proxy);
				}
				NewParticle->SetPhysicsProxy(Proxy);
			});

		MEvolution->SetPreIntegrateCallback(
			[this](FReal Dt)
			{
				OnEvolutionPreIntegrate(Dt);
			});

		MEvolution->SetPostIntegrateCallback(
			[this](FReal Dt)
			{
				OnEvolutionPostIntegrate(Dt);
			});

		MEvolution->SetPreSolveCallback(
			[this](FReal Dt)
			{
				for (ISimCallbackObject* Callback : SimCallbackObjects)
				{
					if (Callback->HasOption(ESimCallbackOptions::PreSolve))
					{
						FScopedTraceSolverCallback TraceCallback(Callback);
						Callback->PreSolve_Internal();
					}
				}

				PreSolveDebugDraw(Dt);
			});

		MEvolution->SetPostSolveCallback(
			[this](FReal Dt)
			{
				for (ISimCallbackObject* Callback : SimCallbackObjects)
				{
					if (Callback->HasOption(ESimCallbackOptions::PostSolve))
					{
						FScopedTraceSolverCallback TraceCallback(Callback);
						Callback->PostSolve_Internal();
					}
				}
			});
	}

	void FPBDRigidsSolver::OnEvolutionPreIntegrate(FReal Dt)
	{
		for (ISimCallbackObject* Callback : SimCallbackObjects)
		{
			if (Callback->HasOption(ESimCallbackOptions::PreIntegrate))
			{
				FScopedTraceSolverCallback TraceCallback(Callback);
				Callback->PreIntegrate_Internal();
			}
		}

		PreIntegrateDebugDraw(Dt);
	}

	void FPBDRigidsSolver::OnEvolutionPostIntegrate(FReal Dt)
	{
		bool bHasIntegrationCompleted = false;
		for (ISimCallbackObject* Callback : SimCallbackObjects)
		{
			if (Callback->HasOption(ESimCallbackOptions::PostIntegrate))
			{
				if (!bHasIntegrationCompleted)
				{
					// If a post integrate callback is called, 
					// we need to wait for all workers threads integrating to finish.
					MEvolution->WaitIntegrationComplete();
					bHasIntegrationCompleted = true;
				}
				
				FScopedTraceSolverCallback TraceCallback(Callback);
				Callback->PostIntegrate_Internal();
			}
		}
	}


#if CHAOS_DEBUG_NAME
	void FPBDRigidsSolver::OnDebugNameChanged()
	{
		MEvolution->SetName(GetDebugName().ToString());
	}
#endif

	FRealSingle MaxBoundsForTree = (FRealSingle)10000;
	FAutoConsoleVariableRef CVarMaxBoundsForTree(
		TEXT("p.MaxBoundsForTree"),
		MaxBoundsForTree,
		TEXT("The max bounds before moving object into a large objects structure. Only applies on object registration")
		TEXT(""),
		ECVF_Default);

	void FPBDRigidsSolver::RegisterObject(FSingleParticlePhysicsProxy* Proxy)
	{
		UE_CHAOS_ASYNC_INITBODY_WRITESCOPELOCK(GetExternalDataLock_External());
		LLM_SCOPE(ELLMTag::ChaosBody);

		UE_LOGF(LogPBDRigidsSolver, Verbose, "FPBDRigidsSolver::RegisterObject()");
		auto& RigidBody_External = Proxy->GetGameThreadAPI();

		if (BroadPhaseConfig.BroadphaseType < FBroadPhaseConfig::TreeAndGrid)
		{
			FSpatialAccelerationIdx Idx = RigidBody_External.SpatialIdx();
			Idx.Bucket = 0;
			RigidBody_External.SetSpatialIdx(Idx);
		}
		else if (RigidBody_External.GetGeometry() && RigidBody_External.GetGeometry()->HasBoundingBox() && RigidBody_External.GetGeometry()->BoundingBox().Extents().Max() >= MaxBoundsForTree)
		{
			RigidBody_External.SetSpatialIdx(FSpatialAccelerationIdx{ 1,0 });
		}
		if (!ensure(Proxy->GetParticle_LowLevel()->IsParticleValid()))
		{
			return;
		}

		// NOTE: Do we really need these lists of proxies if we can just
		// access them through the GTParticles list?
		
		// Evolution should be valid at this point, adding this check to avoid a server crash
		if (!ensure(GetEvolution()))
		{
			UE_LOGF(LogPBDRigidsSolver, Warning, "FPBDRigidsSolver::RegisterObject: Regiester a proxy while Evolution is still null.");
			return;
		}

		RigidBody_External.SetUniqueIdx(GetEvolution()->GenerateUniqueIdx());
		TrackGTParticle_External(*Proxy->GetParticle_LowLevel());	//todo: remove this
		//FParticlePropertiesData& RemoteParticleData = *DirtyPropertiesManager->AccessProducerBuffer()->NewRemoteParticleProperties();
		//FShapeRemoteDataContainer& RemoteShapeContainer = *DirtyPropertiesManager->AccessProducerBuffer()->NewRemoteShapeContainer();

		Proxy->SetSolver(this);
		Proxy->GetParticle_LowLevel()->SetProxy(Proxy);
		AddDirtyProxy(Proxy);

		UpdateParticleInAccelerationStructure_External(Proxy->GetParticle_LowLevel(), EPendingSpatialDataOperation::Add);
	}

	int32 LogCorruptMap = 0;
	FAutoConsoleVariableRef CVarLogCorruptMap(TEXT("p.LogCorruptMap"), LogCorruptMap, TEXT(""));

	void FPBDRigidsSolver::ApplyCallbacks_Internal()
	{
		Super::ApplyCallbacks_Internal();

		if (MRewindCallback) // Note: Don't use ShouldApplyRewindCallbacks() here since we want this called even if we don't have RewindData enabled.
		{
			if (bGameThreadFrozen)
			{ 
				MRewindCallback->ApplyCallbacks_Internal(GetCurrentFrame(), SimCallbackObjects);
			}
		}
	}

	FSolverSerializer& FPBDRigidsSolver::GetSerializer()
	{
		return Serializer;
	}

	bool bResimDelayDestroyObject = true;
	FAutoConsoleVariableRef CVarResimDelayDestroyObject(TEXT("p.Resim.DelayDestroyObject"), bResimDelayDestroyObject, TEXT("Keep physics objects and physics proxy available until after they go out of rewind history. Disabled by default due to not fully implemented."));

	void FPBDRigidsSolver::ClearCollisionEventsForProxy(const IPhysicsProxyBase* Proxy)
	{
		// @todo(chaos): Race Condition: In async mode UnregisterObject may be called while the physics thread is
		// running, and perhaps just about to dispatch events. The command below will not execute until the 
		// next frame, so we may still get collision events referencing our owner (via the UserData). 
		// See FEventDefaults::RegisterCollisionEvent. The logic for collecting events checks Proxy->GetMarkDeleted(), 
		// but the events may have already been collected, and that flag is also not intended for PT access and is
		// not thread safe.
		// For now we require the user to check GetMarkDeleted() in their collision handler to avoid the race condition.

		GetEventManager()->template ClearEvents<FCollisionEventData>(EEventType::Collision, [Proxy]
			(FCollisionEventData& EventDataInOut)
			{
				FCollisionDataArray const& CollisionData = EventDataInOut.CollisionData.AllCollisionsArray;
				if (CollisionData.Num() > 0)
				{
					check(Proxy);
					TArray<int32> const* const CollisionIndices = EventDataInOut.PhysicsProxyToCollisionIndices.PhysicsProxyToIndicesMap.Find(Proxy);
					if (CollisionIndices)
					{
						EventDataInOut.PhysicsProxyToCollisionIndices.PhysicsProxyToIndicesMap.Remove(Proxy);
					}
				}
			});
	}

	void FPBDRigidsSolver::UnregisterObject(FSingleParticlePhysicsProxy* Proxy)
	{
		UE_CHAOS_ASYNC_INITBODY_WRITESCOPELOCK(GetExternalDataLock_External());
		UE_LOGF(LogPBDRigidsSolver, Verbose, "FPBDRigidsSolver::UnregisterObject()");

		// Early out if already marked deleted
		if (Proxy->GetMarkedDeleted())
		{
			return;
		}

		PullResultsManager->RemoveProxy_External(Proxy);

		ClearGTParticle_External(*Proxy->GetParticle_LowLevel());	//todo: remove this

		UpdateParticleInAccelerationStructure_External(Proxy->GetParticle_LowLevel(), EPendingSpatialDataOperation::Delete);

		// remove the proxy from the invalidation list
		RemoveDirtyProxy(Proxy);

		// mark proxy timestamp so we avoid trying to pull from sim after deletion
		Proxy->MarkDeleted();

		// Null out the particle's proxy pointer
		Proxy->GetParticle_LowLevel()->SetProxy(nullptr);	//todo: use TUniquePtr for better ownership

		// Remove the proxy from the GT proxy map
		FUniqueIdx UniqueIdx = Proxy->GetGameThreadAPI().UniqueIdx();

		// Notify GT listeners before the proxy is enqueued for destruction
		EventParticleUnregisteredGT.Broadcast(Proxy, UniqueIdx);

		{
			// The collision manager is also modified from FPBDRigidsSolver::PushPhysicsState which uses the 
			// MarshallingManager lock so we need to lock this here as well.
			// @todo(chaos): should we lock this for the whole function?
			Chaos::Private::FChaosAsyncInitBodyWriteScopeLock MarshallingManagerLock(MarshallingManager.GetMarshallingManagerLock());
			FIgnoreCollisionManager& CollisionManager = GetEvolution()->GetBroadPhase().GetIgnoreCollisionManager();
			int32 ExternalTimestamp = GetMarshallingManager().GetExternalTimestamp_External();
			FIgnoreCollisionManager::FDeactivationSet& PendingMap = CollisionManager.GetPendingDeactivationsForGameThread(ExternalTimestamp);
			PendingMap.Add(UniqueIdx);
		}

		// Enqueue a command to remove the particle and delete the proxy
		EnqueueCommandScheduled_External(GetMarshallingManager().GetInternalStep_External(), [Proxy, UniqueIdx, this]()
		{
			UE_LOGF(LogPBDRigidsSolver, Verbose, "FPBDRigidsSolver::UnregisterObject() ~ Dequeue");

			// Remove stale events for particles that no longer exist
			// See comments in ClearCollisionEventsForProxy for potential race condition
			ClearCollisionEventsForProxy(Proxy);

			FGeometryParticleHandle* Handle = Proxy->GetHandle_LowLevel();

			//If particle was created and destroyed before commands were enqueued just skip. I suspect we can skip entire lambda, but too much code to verify right now
			if(Handle)
			{
				//Disable until particle is finally destroyed
				GetEvolution()->DisableParticle(Handle);
			}

			int32 DelayDestroyFrames = 0;
			if (MRewindData)
			{
				if (!MRewindData->IsResim())
				{
					MRewindData->DisableInHistoryFromFrame(Proxy, INDEX_NONE);
				}

				if (bResimDelayDestroyObject)
				{
					DelayDestroyFrames = MRewindData->Capacity();
				}
			}

			PendingDestroyPhysicsProxy.AddUnique(FPendingDestroyInfo{Proxy, GetCurrentFrame() + DelayDestroyFrames, Handle, UniqueIdx});

			// Remove the proxy for this particle
			if (SingleParticlePhysicsProxies_PT.IsValidIndex(UniqueIdx.Idx))
			{
				SingleParticlePhysicsProxies_PT.RemoveAt(UniqueIdx.Idx);
			}
		});
	}

	FDelegateHandle FPBDRigidsSolver::AddParticleUnregisteredGTCallback(FSolverParticleUnregisteredGT::FDelegate InDelegate)
	{
		return EventParticleUnregisteredGT.Add(InDelegate);
	}

	bool FPBDRigidsSolver::RemoveParticleUnregisteredGTCallback(FDelegateHandle InHandle)
	{
		return EventParticleUnregisteredGT.Remove(InHandle);
	}

	void FPBDRigidsSolver::RegisterObject(FGeometryCollectionPhysicsProxy* InProxy)
	{
		LLM_SCOPE(ELLMTag::ChaosBody);
		UE_LOGF(LogPBDRigidsSolver, Verbose, "FPBDRigidsSolver::RegisterObject(FGeometryCollectionPhysicsProxy*)");
		InProxy->SetSolver(this);
		InProxy->Initialize(GetEvolution());
		InProxy->NewData(); // Buffers data on the proxy.

		// Need to immediately add to the SQ to make sure that the GC is in the external SQ even after the SQ's flip.
		for (const TUniquePtr<FPBDRigidParticle>& Particle : InProxy->GetUnorderedParticles_External())
		{
			if (Particle && !Particle->Disabled())
			{
				UpdateParticleInAccelerationStructure_External(Particle.Get(), EPendingSpatialDataOperation::Add);
			}
		}

		// There used to be an EnqueueCommandImmediate here to push the initialization of the GC
		// onto the physics thread. We should use AddDirtyProxy here instead to better match up with
		// the initialization order done in the ProcessSinglePushedData_Internal. Using EnqueueCommandImmediate
		// will cause the GC to be initialized after constraints which would preclude it from being
		// used with joint constraints that get spawned on level load with the GC.
		AddDirtyProxy(InProxy);
	}
	
	void FPBDRigidsSolver::UnregisterObject(FGeometryCollectionPhysicsProxy* InProxy)
	{
		check(InProxy);

		// Early out if already marked deleted
		if (InProxy->GetMarkedDeleted())
		{
			return;
		}

		// mark proxy timestamp so we avoid trying to pull from sim after deletion
		InProxy->MarkDeleted();

		RemoveDirtyProxy(InProxy);
		InProxy->OnUnregisteredFromSolver();

		// Particles are removed from acceleration structure in FPhysScene_Chaos::RemoveObject.

		EnqueueCommandScheduled_External(GetMarshallingManager().GetInternalStep_External(), [InProxy, this]()
		{
			// Remove stale events for particles that no longer exist
			// See comments in ClearCollisionEventsForProxy for potential race condition
			ClearCollisionEventsForProxy(InProxy);

			GeometryCollectionPhysicsProxies_Internal.RemoveSingle(InProxy);
			InProxy->SyncBeforeDestroy();

			int32 DelayDestroyFrames = 0;
			if (MRewindData)
			{
				if (!MRewindData->IsResim())
				{
					MRewindData->DisableInHistoryFromFrame(InProxy, INDEX_NONE);
				}

				if (bResimDelayDestroyObject)
				{
					DelayDestroyFrames = MRewindData->Capacity();
				}
			}

			PendingDestroyGeometryCollectionPhysicsProxy.AddUnique(FPendingProxyDestroyInfo<FGeometryCollectionPhysicsProxy>{ InProxy, GetCurrentFrame() + DelayDestroyFrames });
		});
	}

	void FPBDRigidsSolver::RegisterObject(FClusterUnionPhysicsProxy* Proxy)
	{
		if (!Proxy)
		{
			return;
		}

		if (FPBDRigidParticle* Particle = Proxy->GetParticle_External())
		{
			if (AccelerationStructureSplitStaticAndDynamic == 1)
			{
				// It needs to be {0, 1}. Things will crash otherwise. I don't know why.
				// Probably similarly to the geometry collection physics proxy.
				Particle->SetSpatialIdx(FSpatialAccelerationIdx{ 0, 1 });
			}
			else
			{
				Particle->SetSpatialIdx(FSpatialAccelerationIdx{ 0, 0 });
			}

			Particle->SetUniqueIdx(GetEvolution()->GenerateUniqueIdx());
			UpdateParticleInAccelerationStructure_External(Particle, EPendingSpatialDataOperation::Add);
		}
		Proxy->SetSolver(this);
		AddDirtyProxy(Proxy);
	}

	void FPBDRigidsSolver::UnregisterObject(FClusterUnionPhysicsProxy* Proxy)
	{
		if (!Proxy)
		{
			return;
		}

		// Early out if already marked deleted
		if (Proxy->GetMarkedDeleted())
		{
			return;
		}

		Proxy->MarkDeleted();
		RemoveDirtyProxy(Proxy);

		if (FPBDRigidParticle* GTParticle = Proxy->GetParticle_External())
		{
			GTParticle->SetProxy(nullptr);
		}

		EnqueueCommandScheduled_External(GetMarshallingManager().GetInternalStep_External(), [Proxy, this]()
			{
				// Remove stale events for particles that no longer exist
				// See comments in ClearCollisionEventsForProxy for potential race condition
				ClearCollisionEventsForProxy(Proxy);

				if (Proxy->GetParticle_Internal())
				{
					MEvolution->DisableParticle(Proxy->GetParticle_Internal());
				}
				ClusterUnionPhysicsProxies_Internal.RemoveSingle(Proxy);
				Proxy->SyncBeforeDestroy();

				int32 DelayDestroyFrames = 0;
				if (MRewindData)
				{
					if (!MRewindData->IsResim())
					{
						MRewindData->DisableInHistoryFromFrame(Proxy, INDEX_NONE);
					}

					if (bResimDelayDestroyObject)
					{
						DelayDestroyFrames = MRewindData->Capacity();
					}
				}
				PendingDestroyClusterUnionProxy.AddUnique(FPendingProxyDestroyInfo<FClusterUnionPhysicsProxy>{ Proxy, GetCurrentFrame() + DelayDestroyFrames });
			}
		);
	}

	void FPBDRigidsSolver::RegisterObject(FJointConstraint* GTConstraint)
	{
		LLM_SCOPE(ELLMTag::ChaosConstraint);
		FJointConstraintPhysicsProxy* JointProxy = new FJointConstraintPhysicsProxy(GTConstraint, nullptr);
		JointProxy->SetSolver(this);

		AddDirtyProxy(JointProxy);
	}

	void FPBDRigidsSolver::UnregisterObject(FJointConstraint* GTConstraint)
	{
		FJointConstraintPhysicsProxy* JointProxy = GTConstraint->GetProxy<FJointConstraintPhysicsProxy>();
		check(JointProxy);

		RemoveDirtyProxy(JointProxy);

		// mark proxy timestamp so we avoid trying to pull from sim after deletion
		GTConstraint->GetProxy()->MarkDeleted();


		GTConstraint->SetProxy(static_cast<FJointConstraintPhysicsProxy*>(nullptr));

		GTConstraint->ReleaseKinematicEndPoint(this);

		JointProxy->DestroyOnGameThread();	//destroy the game thread portion of the proxy
		
		// Finish de-registration on the physics thread...
		EnqueueCommandImmediate([JointProxy, this]()
		{
			//TODO: consider deferring this so that rewind can resim with joint until moment of destruction
			//For now we assume this always comes from server update
			if(FRewindData* RewindData = GetRewindData())
			{
				RewindData->RemoveObject(JointProxy->GetHandle());
			}

			JointProxy->DestroyOnPhysicsThread(this);
			JointConstraintPhysicsProxies_Internal.RemoveSingle(JointProxy);
			delete JointProxy;
		});
	}

	void FPBDRigidsSolver::RegisterObject(FCharacterGroundConstraint* GTConstraint)
	{
		LLM_SCOPE(ELLMTag::ChaosConstraint);
		FCharacterGroundConstraintProxy* ConstraintProxy = new FCharacterGroundConstraintProxy(GTConstraint);
		ConstraintProxy->SetSolver(this);

		AddDirtyProxy(ConstraintProxy);
	}

	void FPBDRigidsSolver::UnregisterObject(FCharacterGroundConstraint* GTConstraint)
	{
		FCharacterGroundConstraintProxy* ConstraintProxy = GTConstraint->GetProxy<FCharacterGroundConstraintProxy>();
		check(ConstraintProxy);

		RemoveDirtyProxy(ConstraintProxy);

		// mark proxy timestamp so we avoid trying to pull from sim after deletion
		GTConstraint->GetProxy()->MarkDeleted();
		GTConstraint->SetProxy(static_cast<FCharacterGroundConstraintProxy*>(nullptr));

		ConstraintProxy->DestroyOnGameThread();	//destroy the game thread portion of the proxy

		// Finish de-registration on the physics thread...
		EnqueueCommandImmediate([ConstraintProxy, this]()
			{
				// TODO: Add character ground constraint to rewind data
				//if (FRewindData* RewindData = GetRewindData())
				//{
				//	RewindData->RemoveObject(ConstraintProxy->GetPhysicsThreadAPI());
				//}

				ConstraintProxy->DestroyOnPhysicsThread(this);
				CharacterGroundConstraintProxies_Internal.RemoveSingle(ConstraintProxy);
				delete ConstraintProxy;
			});
	}




	void FPBDRigidsSolver::RegisterObject(FSuspensionConstraint* GTConstraint)
	{
		LLM_SCOPE(ELLMTag::ChaosConstraint);
		FSuspensionConstraintPhysicsProxy* SuspensionProxy = new FSuspensionConstraintPhysicsProxy(GTConstraint, nullptr);
		SuspensionProxy->SetSolver(this);

		AddDirtyProxy(SuspensionProxy);
	}

	void FPBDRigidsSolver::UnregisterObject(FSuspensionConstraint* GTConstraint)
	{
		FSuspensionConstraintPhysicsProxy* SuspensionProxy = GTConstraint->GetProxy<FSuspensionConstraintPhysicsProxy>();
		check(SuspensionProxy);

		// mark proxy timestamp so we avoid trying to pull from sim after deletion
		SuspensionProxy->MarkDeleted();

		RemoveDirtyProxy(SuspensionProxy);

		GTConstraint->SetProxy(static_cast<FSuspensionConstraintPhysicsProxy*>(nullptr));

		FParticlesType* InParticles = &GetParticles();
		SuspensionProxy->DestroyOnGameThread();	//destroy the game thread portion of the proxy

		// Finish registration on the physics thread...
		EnqueueCommandImmediate([InParticles, SuspensionProxy, this]()
			{
				SuspensionProxy->DestroyOnPhysicsThread(this);
				delete SuspensionProxy;
			});
	}

	void FPBDRigidsSolver::SetSuspensionTarget(FSuspensionConstraint* GTConstraint, const FVector& TargetPos, const FVector& Normal, bool Enabled, const IPhysicsProxyBase* GroundProxy)
	{
		EnsureIsInPhysicsThreadContext();
		FSuspensionConstraintPhysicsProxy* SuspensionProxy = GTConstraint->GetProxy<FSuspensionConstraintPhysicsProxy>();
		check(SuspensionProxy);
		SuspensionProxy->UpdateTargetOnPhysicsThread(this, TargetPos, GroundProxy, Normal, Enabled);
	}

	void FPBDRigidsSolver::EnableRewindCapture(int32 NumFrames, bool InUseCollisionResimCache, TUniquePtr<IRewindCallback>&& RewindCallback)
	{
		SetRewindCallback(MoveTemp(RewindCallback));
		EnableRewindCapture(NumFrames, InUseCollisionResimCache);
	}

	void FPBDRigidsSolver::EnableRewindCapture(int32 NumFrames, bool InUseCollisionResimCache)
	{
		SetUseCollisionResimCache(InUseCollisionResimCache);
		EnableRewindCapture(NumFrames);
	}

	void FPBDRigidsSolver::EnableRewindCapture(int32 NumFrames)
	{
		//TODO: this function calls both internal and external - sort of assumed during initialization. Should decide what thread it's called on and mark it as either external or internal
		if (MRewindData.IsValid())
		{
			MRewindData->Init(((FPBDRigidsSolver*)this), NumFrames, GetCurrentFrame());
		}
		else
		{
			MRewindData = MakeUnique<FRewindData>(((FPBDRigidsSolver*)this), NumFrames, GetCurrentFrame());
		}

		const int32 NumFramesSet = GetRewindData() != nullptr ? GetRewindData()->Capacity() : NumFrames;
		MarshallingManager.SetHistoryLength_Internal(NumFramesSet);
		MEvolution->SetRewindData(GetRewindData());

		if (MRewindCallback)
		{
			MRewindCallback->RewindData = GetRewindData();
		}

		UpdateIsDeterministic();

		UE_LOGF(LogChaos, Log, "PBDRigidsSolver::EnableRewindCapture - Starting physics data history caching for rewind / resimulation. History Size: %d. Supported Latency: %f. TickRate: %d. ", NumFramesSet, FPBDRigidsSolver::GetPhysicsHistoryTimeLength(), FMath::RoundToInt32(1.0f / GetAsyncDeltaTime()));
	}

	void FPBDRigidsSolver::EnableRewindCapture()
	{
		EnableRewindCapture(0);
	}

	void FPBDRigidsSolver::Reset()
	{
		UE_LOGF(LogPBDRigidsSolver, Verbose, "PBDRigidsSolver::Reset()");

		MTime = 0;
		MLastDt = 0.0f;
		CurrentFrame = 0;
		SetMaxDeltaTime_External(1.0f);
		SetMinDeltaTime_External(UE_SMALL_NUMBER);
		SetMaxSubSteps_External(1);
		MEvolution = TUniquePtr<FPBDRigidsEvolution>(new FPBDRigidsEvolution(Particles, SimMaterials, &MidPhaseModifiers, &CCDModifiers, &StrainModifiers, &ContactModifiers, BufferMode == EMultiBufferMode::Single)); 

		PerSolverField = MakeUnique<FPerSolverFieldSystem>();

		//todo: do we need this?
		//MarshallingManager.Reset();

		MRewindData = nullptr;
		if (MRewindCallback)
		{
			MRewindCallback->RewindData = nullptr;
		}
		MarshallingManager.SetHistoryLength_Internal(0);

		FEventDefaults::RegisterSystemEvents(*GetEventManager());
	}

	void FPBDRigidsSolver::ChangeBufferMode(EMultiBufferMode InBufferMode)
	{
		// This seems unused inside the solver? #BH
		BufferMode = InBufferMode;

		SetThreadingMode_External(BufferMode == EMultiBufferMode::Single ? EThreadingModeTemp::SingleThread : EThreadingModeTemp::TaskGraph);
	}

	void FPBDRigidsSolver::StartingSceneSimulation()
	{
		LLM_SCOPE(ELLMTag::Chaos);
		QUICK_SCOPE_CYCLE_COUNTER(STAT_StartedSceneSimulation);

		GetEvolution()->GetBroadPhase().GetIgnoreCollisionManager().PopStorageData_Internal(GetEvolution()->LatestExternalTimestampConsumed_Internal);
	}

	void FPBDRigidsSolver::DestroyPendingProxies_Internal()
	{
		MarshallingManager.GetCurrentPullData_Internal()->DirtyClusterUnions.Reset();
		
		for (int32 Idx = PendingDestroyClusterUnionProxy.Num() - 1; Idx >= 0; --Idx)
		{
			FPendingProxyDestroyInfo<FClusterUnionPhysicsProxy>& Info = PendingDestroyClusterUnionProxy[Idx];

			// Are we on or after the frame where we want to destroy the proxy
			// Note: GetCurrentFrame() - 1 because this happens at the end of the physics frame after frame number has been incremented already
			if (Info.DestroyOnStep <= (GetCurrentFrame() - 1) || IsShuttingDown()) 
			{
				if (Info.Proxy != nullptr)
				{
					// Callback for Unregister PhysicsObject
					if (PhysicsObjectUnregistrationWatchers.Num() > 0)
					{
						for (ISimCallbackObject* Callback : PhysicsObjectUnregistrationWatchers)
						{
							Callback->OnPhysicsObjectUnregistered_Internal(Info.Proxy->GetPhysicsObjectHandle());
						}
					}

					if (FPBDRigidsEvolutionGBF* Evolution = GetEvolution())
					{
						// Remove the cluster union this proxy manages which will in turn also the destroy the necessary particles.
						Evolution->GetRigidClustering().GetClusterUnionManager().DestroyClusterUnion(Info.Proxy->GetClusterUnionIndex());
					}

					delete Info.Proxy;
				}

				PendingDestroyClusterUnionProxy.RemoveAtSwap(Idx, EAllowShrinking::No);
			}
		}

		// If we have any callback objects watching for particle deregistrations,
		// send an array of proxies that are about to be deleted.
		if (UnregistrationWatchers.Num() > 0)
		{
			TArray<TTuple<FUniqueIdx, FSingleParticlePhysicsProxy*>> Proxies;
			Proxies.Reserve(PendingDestroyPhysicsProxy.Num());
			for (FPendingDestroyInfo& Info : PendingDestroyPhysicsProxy)
			{
				Proxies.Add({ Info.UniqueIdx, Info.Proxy });
			}
			for (ISimCallbackObject* Callback : UnregistrationWatchers)
			{
				Callback->OnParticleUnregistered_Internal(Proxies);
			}
		}

		// Do the actual destruction
		for(int32 Idx = PendingDestroyPhysicsProxy.Num() - 1; Idx >= 0; --Idx)
		{
			FPendingDestroyInfo& Info = PendingDestroyPhysicsProxy[Idx];
			if(Info.DestroyOnStep <= (GetCurrentFrame() - 1) || IsShuttingDown()) // Note: GetCurrentFrame() - 1 because this happens at the end of the physics frame after frame number has been incremented already
			{
				// Callback for Unregister PhysicsObject
				if (PhysicsObjectUnregistrationWatchers.Num() > 0)
				{
					for (ISimCallbackObject* Callback : PhysicsObjectUnregistrationWatchers)
					{
						Callback->OnPhysicsObjectUnregistered_Internal(Info.Proxy->GetPhysicsObject());
					}
				}

				// finally let's release the unique index
				GetEvolution()->ReleaseUniqueIdx(Info.UniqueIdx);

				if (Info.Handle)
				{
					// Use the handle to destroy the particle data
					GetEvolution()->DestroyParticle(Info.Handle);
				}
				Info.Proxy->SetHandle(nullptr);

				delete Info.Proxy;
				PendingDestroyPhysicsProxy.RemoveAtSwap(Idx, EAllowShrinking::No);
			}
		}

		MarshallingManager.GetCurrentPullData_Internal()->DirtyGeometryCollections.Reset();

		for (int32 Idx = PendingDestroyGeometryCollectionPhysicsProxy.Num() - 1; Idx >= 0; --Idx)
		{
			FPendingProxyDestroyInfo<FGeometryCollectionPhysicsProxy>& Info = PendingDestroyGeometryCollectionPhysicsProxy[Idx];

			// Are we on or after the frame where we want to destroy the proxy
			// Note: GetCurrentFrame() - 1 because this happens at the end of the physics frame after frame number has been incremented already
			if (Info.DestroyOnStep <= (GetCurrentFrame() - 1) || IsShuttingDown())
			{
				if (Info.Proxy != nullptr)
				{
					GetEvolution()->GetRigidClustering().CleanupInternalClustersForProxies(MakeArrayView(reinterpret_cast<IPhysicsProxyBase**>(&Info.Proxy), 1));

					// Callback for Unregister PhysicsObject
					if (PhysicsObjectUnregistrationWatchers.Num() > 0)
					{
						TArray<FPhysicsObjectHandle> PhysicsObjects = Info.Proxy->GetAllPhysicsObjects();
						for (FConstPhysicsObjectHandle PhysicsObject : PhysicsObjects)
						{
							for (ISimCallbackObject* Callback : PhysicsObjectUnregistrationWatchers)
							{
								Callback->OnPhysicsObjectUnregistered_Internal(PhysicsObject);
							}
						}
					}

					// Removing the geometry collection from the solver a bit delayed. This lets the cluster union do its cleanup first before
					// the geometry collection if they're all being destroyed at the same time.
					Info.Proxy->OnRemoveFromSolver(this);
					delete Info.Proxy;
				}

				PendingDestroyGeometryCollectionPhysicsProxy.RemoveAtSwap(Idx, EAllowShrinking::No);
			}
		}
	}

	void FPBDRigidsSolver::SetVelocityBoundsExpansion(const FReal BoundsVelocityMultiplier, const FReal MaxBoundsVelocityExpansion)
	{ 
		GetEvolution()->GetCollisionConstraints().SetVelocityBoundsExpansion(BoundsVelocityMultiplier, MaxBoundsVelocityExpansion);
	}

	void FPBDRigidsSolver::SetVelocityBoundsExpansionMACD(const FReal BoundsVelocityMultiplier, const FReal MaxBoundsVelocityExpansion)
	{
		GetEvolution()->GetCollisionConstraints().SetVelocityBoundsExpansionMACD(BoundsVelocityMultiplier, MaxBoundsVelocityExpansion);
	}

	void FPBDRigidsSolver::PrepareAdvanceBy(const FReal DeltaTime)
	{
		// Handle runtime cvar changes for debugging
		ApplyCVars();

		UE_LOGF(LogPBDRigidsSolver, Verbose, "PBDRigidsSolver::Tick(%3.5f)", DeltaTime);
		MLastDt = DeltaTime;
		EventPreSolve.Broadcast(DeltaTime);

		StartingSceneSimulation();
	}

	void FPBDRigidsSolver::AdvanceSolverBy(const FSubStepInfo& SubStepInfo)
	{
		const FReal StartSimTime = GetSolverTime();

#if CHAOS_DEBUG_DRAW
		ChaosDD::Private::FChaosDDTimelineContext DDContext;
		bool bEnableDebugDrawCapture = !GetEvolution()->IsResimming();
		if (bEnableDebugDrawCapture)
		{
			DDContext.BeginFrame(CDDFrameTimeline, StartSimTime, GetLastDt());
		}
#endif

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if(IsNetworkPhysicsPredictionEnabled() && CanDebugNetworkPhysicsPrediction())
		{
			UE_LOGF(LogChaos, Log, "-> Simulating Frame = %d", CurrentFrame);
		}
#endif

		AdvanceOneTimeStepTask(this, MLastDt, SubStepInfo).DoWork();

		if (MLastDt > 0)
		{
			SCOPE_CYCLE_COUNTER(STAT_FinalizePullData);
			//pass information back to external thread
			//we skip dt=0 case because sync data should be identical if dt = 0
			MarshallingManager.FinalizePullData_Internal(MEvolution->LatestExternalTimestampConsumed_Internal, StartSimTime, MLastDt);
		}

		if(SubStepInfo.Step == SubStepInfo.NumSteps - 1)
		{
			SCOPE_CYCLE_COUNTER(STAT_DestroyPendingProxies);
			//final step so we can destroy proxies
			DestroyPendingProxies_Internal();
		}

#if CHAOS_DEBUG_DRAW
		if (bEnableDebugDrawCapture)
		{
			DDContext.EndFrame();
		}
#endif
	}

#if CHAOS_DEBUG_DRAW
	void FPBDRigidsSolver::SetDebugDrawScene(const ChaosDD::Private::FChaosDDScenePtr& InCDDScene)
	{
		CDDScene = InCDDScene;
		CDDFrameTimeline.Reset();

		if (CDDScene.IsValid())
		{
			CDDFrameTimeline = CDDScene->CreateTimeline(FString::Format(TEXT("{0} {1}"), { CDDScene->GetName(), "Physics Frame" }));
		}

		GetEvolution()->SetDebugDrawScene(InCDDScene);
	}
#endif

	void FPBDRigidsSolver::SetExternalTimestampConsumed_Internal(const int32 Timestamp)
	{
		MEvolution->LatestExternalTimestampConsumed_Internal = Timestamp;
	}

	void FPBDRigidsSolver::SyncEvents_GameThread()
	{
		GetEventManager()->DispatchEvents();
	}

	int32 LogDirtyParticles = 0;
	FAutoConsoleVariableRef CVarLogDirtyParticles(TEXT("p.LogDirtyParticles"), LogDirtyParticles, TEXT("Logs out which particles are dirty every frame"));

	void FPBDRigidsSolver::PushPhysicsState(const FReal DeltaTime, const int32 NumSteps, const int32 NumExternalSteps)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_PushPhysicsState);
		ensure(NumSteps > 0);
		ensure(NumExternalSteps > 0);
		//TODO: interpolate some data based on num steps

		UE_CHAOS_ASYNC_INITBODY_WRITESCOPELOCK(MarshallingManager.GetMarshallingManagerLock());
		FPushPhysicsData* PushData = MarshallingManager.GetProducerData_External();
		const FReal DynamicsWeight = FReal(1) / FReal(NumExternalSteps);
		FDirtySet* DirtyProxiesData = &PushData->DirtyProxiesDataBuffer;
		FDirtyPropertiesManager* Manager = &PushData->DirtyPropertiesManager;

		Manager->PrepareBuckets(DirtyProxiesData->GetDirtyProxyBucketInfo());
		Manager->SetNumShapes(DirtyProxiesData->NumDirtyShapes());
		FShapeDirtyData* ShapeDirtyData = DirtyProxiesData->GetShapesDirtyData();

		DirtyProxiesData->ParallelForEachProxy([this, DynamicsWeight, Manager, ShapeDirtyData](int32 DataIdx, FDirtyProxy& Dirty)
		{
			switch(Dirty.Proxy->GetType())
			{
			case EPhysicsProxyType::SingleParticleProxy:
			{
				auto Proxy = static_cast<FSingleParticlePhysicsProxy*>(Dirty.Proxy);
				auto Particle = Proxy->GetParticle_LowLevel();
				if(auto Rigid = Particle->CastToRigidParticle())
				{
					Rigid->ApplyDynamicsWeight(DynamicsWeight);
				}

				Particle->PrepareBVH();
				Particle->LockGeometry();
				Particle->SyncRemoteData(*Manager, DataIdx, Dirty.PropertyData, Dirty.ShapeDataIndices, ShapeDirtyData);
				Proxy->ClearAccumulatedData();
				Proxy->ResetDirtyIdx();
				break;
			}
			case EPhysicsProxyType::GeometryCollectionType:
			{
				auto Proxy = static_cast<FGeometryCollectionPhysicsProxy*>(Dirty.Proxy);
				Proxy->PushStateOnGameThread(this);
				Proxy->ResetDirtyIdx();
				break;
			}
			case EPhysicsProxyType::ClusterUnionProxy:
			{
				FClusterUnionPhysicsProxy* Proxy = static_cast<FClusterUnionPhysicsProxy*>(Dirty.Proxy);
				FClusterUnionPhysicsProxy::FExternalParticle* Particle = Proxy->GetParticle_External();
				Particle->LockGeometry();
				Proxy->SyncRemoteData(*Manager, DataIdx, Dirty.PropertyData);
				Proxy->ClearAccumulatedData();
				Proxy->ResetDirtyIdx();
				break;
			}
			case EPhysicsProxyType::JointConstraintType:
			{
				auto Proxy = static_cast<FJointConstraintPhysicsProxy*>(Dirty.Proxy);
				Proxy->PushStateOnGameThread(*Manager, DataIdx, Dirty.PropertyData);
				Proxy->ResetDirtyIdx();
				break;
			}
			case EPhysicsProxyType::SuspensionConstraintType:
			{
				auto Proxy = static_cast<FSuspensionConstraintPhysicsProxy*>(Dirty.Proxy);
				Proxy->PushStateOnGameThread(*Manager, DataIdx, Dirty.PropertyData);
				Proxy->ResetDirtyIdx();
				break;
			}
			case EPhysicsProxyType::CharacterGroundConstraintType:
			{
				auto Proxy = static_cast<FCharacterGroundConstraintProxy*>(Dirty.Proxy);
				Proxy->PushStateOnGameThread(*Manager, DataIdx, Dirty.PropertyData);
				Proxy->ResetDirtyIdx();
				break;
			}

			default:
			ensure(0 && TEXT("Unknown proxy type in physics solver."));
			}
		});

		if(!!LogDirtyParticles)
		{
			int32 NumJoints = 0;
			int32 NumSuspension = 0;
			int32 NumCharacterGroundConstraints = 0;
			int32 NumParticles = 0;
			UE_LOGF(LogChaos, Warning, "LogDirtyParticles:");
			DirtyProxiesData->ForEachProxy([&NumJoints, &NumSuspension, &NumCharacterGroundConstraints, &NumParticles](int32 DataIdx, FDirtyProxy& Dirty)
			{
				switch (Dirty.Proxy->GetType())
				{
					case EPhysicsProxyType::SingleParticleProxy:
					{
						auto Proxy = static_cast<FSingleParticlePhysicsProxy*>(Dirty.Proxy);
#if CHAOS_DEBUG_NAME
						UE_LOGF(LogChaos, Warning, "\t%ls", **Proxy->GetParticle_LowLevel()->DebugName());
#endif
						++NumParticles;
						break;
					}
					case EPhysicsProxyType::JointConstraintType:
					{
						auto Proxy = static_cast<FJointConstraintPhysicsProxy*>(Dirty.Proxy);
						++NumJoints;
						break;
					}
					case EPhysicsProxyType::SuspensionConstraintType:
					{
						++NumSuspension;
						break;
					}
					case EPhysicsProxyType::CharacterGroundConstraintType:
					{
						++NumCharacterGroundConstraints;
						break;
					}

					default: break;
				}
			});

			UE_LOGF(LogChaos, Warning, "Num Particles:%d Num Shapes:%d Num Joints:%d Num Suspensions:%d Num CharGround:%d", NumParticles, DirtyProxiesData->NumDirtyShapes(), NumJoints, NumSuspension, NumCharacterGroundConstraints);
		}

		GetEvolution()->GetBroadPhase().GetIgnoreCollisionManager().PushProducerStorageData_External(MarshallingManager.GetExternalTimestamp_External());

		if (!IsShuttingDown())
		{
			for (ISimCallbackObject* Callback : InjectInputsWatchers_External)
			{
				FScopedTraceSolverCallback TraceCallback(Callback);
				Callback->InjectInputs_External(MarshallingManager.GetInternalStep_External(), NumSteps);
			}

			if (MRewindCallback)
			{
				MRewindCallback->InjectInputs_External(MarshallingManager.GetInternalStep_External(), NumSteps);
			}
		}

		MarshallingManager.Step_External(DeltaTime, NumSteps, GetSolverSubstep_External());
	}

	void FPBDRigidsSolver::ProcessSinglePushedData_Internal(FPushPhysicsData& PushData)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_ProcessSinglePushedData_Internal);

		FRewindData* RewindData = GetRewindData();

		FDirtySet* DirtyProxiesData = &PushData.DirtyProxiesDataBuffer;
		FDirtyPropertiesManager* Manager = &PushData.DirtyPropertiesManager;
		FShapeDirtyData* ShapeDirtyData = DirtyProxiesData->GetShapesDirtyData();
		FReal ExternalDt = PushData.ExternalDt;

		TArray<FSingleParticlePhysicsProxy*> RegisteredProxies;

		auto ProcessProxyPT = [Manager, ShapeDirtyData, RewindData, ExternalDt, &RegisteredProxies, this](auto& Proxy,int32 DataIdx,FDirtyProxy& Dirty,const auto& CreateHandleFunc)
		{
			const bool bIsNew = !Proxy->IsInitialized();
			if(bIsNew)
			{
				const auto* NonFrequentData = Dirty.PropertyData.FindNonFrequentData(*Manager,DataIdx);
				const FUniqueIdx* UniqueIdx = NonFrequentData ? &NonFrequentData->UniqueIdx() : nullptr;
				Proxy->SetHandle(CreateHandleFunc(UniqueIdx));

				auto Handle = Proxy->GetHandle_LowLevel();
				Handle->GTGeometryParticle() = Proxy->GetParticle_LowLevel();

				// If anybody's listening for proxy creations, build a list of proxies that have been
				// added in this step.
				if (RegistrationWatchers.Num() > 0)
				{
					RegisteredProxies.Add(Proxy);
				}

				// Track proxies
				if (UniqueIdx)
				{
					SingleParticlePhysicsProxies_PT.EmplaceAt(UniqueIdx->Idx, Proxy);
				}
			}

			if(Proxy->GetHandle_LowLevel())
			{
				if (RewindData)
				{
					RewindData->PushGTDirtyData(*Manager, DataIdx, Dirty, ShapeDirtyData);
				}
				Proxy->PushToPhysicsState(*Manager, DataIdx, Dirty, ShapeDirtyData, ExternalDt);
			}
			else
			{
				//The only valid time for a handle to not exist is during a resim, when the proxy was already deleted
				//Another way would be to sanitize pending push data, but this would be expensive
				ensure(RewindData && RewindData->IsResim());
			}

			if(bIsNew)
			{
				auto Handle = Proxy->GetHandle_LowLevel();
				if(auto Rigid = Handle->CastToRigidParticle())
				{
					Rigid->SetPreObjectStateLowLevel(Rigid->ObjectState());	//created this frame so pre is the initial value
				}
				Handle->SetPhysicsProxy(Proxy);
				GetEvolution()->RegisterParticle(Handle);
				Proxy->SetInitialized(GetCurrentFrame());
			}
		};

		//need to create new particle handles
		int32 NumInitializedGCProxies = 0;
		DirtyProxiesData->ForEachProxy([this, &ProcessProxyPT, Manager, &NumInitializedGCProxies](int32 DataIdx,FDirtyProxy& Dirty)
		{
			if(Dirty.Proxy->GetIgnoreDataOnStep_Internal() != CurrentFrame)
			{
				switch(Dirty.Proxy->GetType())
				{
					case EPhysicsProxyType::SingleParticleProxy:
					{
						FSingleParticlePhysicsProxy* Proxy = static_cast<FSingleParticlePhysicsProxy*>(Dirty.Proxy);
						ProcessProxyPT(Proxy, DataIdx, Dirty, [this, &Dirty](const FUniqueIdx* UniqueIdx) -> TGeometryParticleHandle<FReal,3>*
						{
							switch (Dirty.PropertyData.GetParticleBufferType())
							{
								case EParticleType::Static: return Particles.CreateStaticParticles(1, UniqueIdx)[0];
								case EParticleType::Kinematic: return Particles.CreateKinematicParticles(1, UniqueIdx)[0];

								case EParticleType::Rigid: 
								{
									if (CVars::bChaosEnableOverrideSingleParticleTypeToCluster)
									{
										return Particles.CreateClusteredParticles(1, UniqueIdx)[0];
									}
									else
									{
										return Particles.CreateDynamicParticles(1, UniqueIdx)[0];
									}
								}
								
								default: check(false); return nullptr;
							}
						});
						break;
					}
			
					case EPhysicsProxyType::GeometryCollectionType:
					{
						FGeometryCollectionPhysicsProxy* Proxy = static_cast<FGeometryCollectionPhysicsProxy*>(Dirty.Proxy);
						if (!Proxy->IsInitializedOnPhysicsThread())
						{
							// Finish registration on the physics thread...
							Proxy->InitializeBodiesPT(this, GetParticles());
							++NumInitializedGCProxies;

							GeometryCollectionPhysicsProxies_Internal.Add(Proxy);
						}
						Proxy->PushToPhysicsState();
						break;
					}
					case EPhysicsProxyType::ClusterUnionProxy:
					{
						FClusterUnionPhysicsProxy* Proxy = static_cast<FClusterUnionPhysicsProxy*>(Dirty.Proxy);
						if (!Proxy->IsInitializedOnPhysicsThread())
						{
							Proxy->Initialize_Internal(this, GetParticles());
							ClusterUnionPhysicsProxies_Internal.Add(Proxy);
						}
						Proxy->PushToPhysicsState(*Manager, DataIdx, Dirty);
						break;
					}
					case EPhysicsProxyType::JointConstraintType:
					case EPhysicsProxyType::SuspensionConstraintType:
					case EPhysicsProxyType::CharacterGroundConstraintType:
					{
						// Pass until after all bodies are created. 
						break;
					}
				}
			}
		});

		//need to create new constraint handles
		DirtyProxiesData->ForEachProxy([this, Manager, RewindData](int32 DataIdx, FDirtyProxy& Dirty)
		{
			if (Dirty.Proxy->GetIgnoreDataOnStep_Internal() != CurrentFrame)
			{
				switch (Dirty.Proxy->GetType())
				{
					case EPhysicsProxyType::JointConstraintType:
					{
						auto JointProxy = static_cast<FJointConstraintPhysicsProxy*>(Dirty.Proxy);
						const bool bIsNew = !JointProxy->IsInitialized();
						if (bIsNew)
						{
							JointConstraintPhysicsProxies_Internal.Add(JointProxy);
							JointProxy->InitializeOnPhysicsThread(this, *Manager, DataIdx, Dirty.PropertyData);
							JointProxy->SetInitialized(GetCurrentFrame());
						}

						//TODO: if we support predicting creation / destruction of joints need to handle null joint case
						if (RewindData)
						{
							RewindData->PushGTDirtyData(*Manager, DataIdx, Dirty, nullptr);
						}
				
						JointProxy->PushStateOnPhysicsThread(this, *Manager, DataIdx, Dirty.PropertyData);

#if UE_WITH_REMOTE_OBJECT_HANDLE
						if (bIsNew)
						{
							
							if (FSerializedDataBufferPtr StateBuffer = Serializer.PopPendingInternalSerializedStateForProxy(JointProxy))
							{
								Serializer.ApplySerializedStateToJointConstraint(JointProxy->GetHandle(), *StateBuffer);
							}
						}
#endif
						break;
					}

					case EPhysicsProxyType::SuspensionConstraintType:
					{
						auto SuspensionProxy = static_cast<FSuspensionConstraintPhysicsProxy*>(Dirty.Proxy);
						const bool bIsNew = !SuspensionProxy->IsInitialized();
						if (bIsNew)
						{
							SuspensionProxy->InitializeOnPhysicsThread(this, *Manager, DataIdx, Dirty.PropertyData);
							SuspensionProxy->SetInitialized();
						}
						SuspensionProxy->PushStateOnPhysicsThread(this, *Manager, DataIdx, Dirty.PropertyData);
						break;
					}

					case EPhysicsProxyType::CharacterGroundConstraintType:
					{
						auto ConstraintProxy = static_cast<FCharacterGroundConstraintProxy*>(Dirty.Proxy);
						const bool bIsNew = !ConstraintProxy->IsInitialized();
						if (bIsNew)
						{
							CharacterGroundConstraintProxies_Internal.Add(ConstraintProxy);
							ConstraintProxy->InitializeOnPhysicsThread(this, *Manager, DataIdx, Dirty.PropertyData);
							ConstraintProxy->SetInitialized(GetCurrentFrame());
						}

						//TODO: Support rewind for character ground constraints
						//if (RewindData)
						//{
						//	RewindData->PushGTDirtyData(*Manager, DataIdx, Dirty, nullptr);
						//}

						ConstraintProxy->PushStateOnPhysicsThread(this, *Manager, DataIdx, Dirty.PropertyData);
						break;
					}
				}
			}
		});

		// If we have callbacks watching for particle registrations, send them the array
		// of newly added proxies
		if (RegistrationWatchers.Num() > 0)
		{
			for (ISimCallbackObject* Callback : RegistrationWatchers)
			{
				Callback->OnParticlesRegistered_Internal(RegisteredProxies);
			}
		}

		//MarshallingManager.FreeData_Internal(&PushData);
	}

	template<typename TRigidParticle>
	bool ShouldUpdateFromSimulation(const TRigidParticle& InRigidParticle)
	{
		if (InRigidParticle.ObjectState() == Chaos::EObjectStateType::Kinematic)
		{
			switch (Chaos::SyncKinematicOnGameThread)
			{
			case 0:
				return false;
			case 1:
				return true;
			default:
				return InRigidParticle.UpdateKinematicFromSimulation();
			}
		}
		// We assume that sleeping/static particles etc won't appear repeatedly (over multiple
		// frames) in the dirty list, so we can safely return true here without incurring unwanted costs.
		return true;
	}

	bool HasSimulationCallback(const ISimCallbackObject* Callback)
	{
		return Callback->HasOption(ESimCallbackOptions::Presimulate) ||
			Callback->HasOption(ESimCallbackOptions::PreSolve) ||
			Callback->HasOption(ESimCallbackOptions::PostSolve) ||
			Callback->HasOption(ESimCallbackOptions::PreIntegrate) ||
			Callback->HasOption(ESimCallbackOptions::PostIntegrate);
	}

	void FPBDRigidsSolver::ProcessPushedData_Internal(FPushPhysicsData& PushData)
	{
		QUICK_SCOPE_CYCLE_COUNTER(ChaosPushData);
		ensure(PushData.InternalStep == CurrentFrame);	//push data was generated for this specific frame

		// Set the current PushData being used on the internal thread
		MarshallingManager.SetConsumerData_Internal(&PushData);

		//update callbacks
		{
			QUICK_SCOPE_CYCLE_COUNTER(Chaos_ProcessPushedData_AddCallbacks);
			SimCallbackObjects.Reserve(SimCallbackObjects.Num() + PushData.SimCallbackObjectsToAdd.Num());
			for(ISimCallbackObject* SimCallbackObject : PushData.SimCallbackObjectsToAdd)
			{
				if (HasSimulationCallback(SimCallbackObject))
				{
					SimCallbackObjects.Add(SimCallbackObject);
				}

				if (SimCallbackObject->HasOption(ESimCallbackOptions::MidPhaseModification))
				{
					MidPhaseModifiers.Add(SimCallbackObject);
				}

				if (SimCallbackObject->HasOption(ESimCallbackOptions::CCDModification))
				{
					CCDModifiers.Add(SimCallbackObject);
				}

				if (SimCallbackObject->HasOption(ESimCallbackOptions::StrainModification))
				{
					StrainModifiers.Add(SimCallbackObject);
				}

				if (SimCallbackObject->HasOption(ESimCallbackOptions::ContactModification))
				{
					ContactModifiers.Add(SimCallbackObject);
				}

				if (SimCallbackObject->HasOption(ESimCallbackOptions::ParticleRegister))
				{
					RegistrationWatchers.Add(SimCallbackObject);
				}

				if (SimCallbackObject->HasOption(ESimCallbackOptions::ParticleUnregister))
				{
					UnregistrationWatchers.Add(SimCallbackObject);
				}

				if (SimCallbackObject->HasOption(ESimCallbackOptions::Rewind))
				{
					if (MRewindCallback)
					{
						MRewindCallback->RegisterRewindableSimCallback_Internal(SimCallbackObject);
					}
				}

				if (SimCallbackObject->HasOption(ESimCallbackOptions::ProcessInputsInternal))
				{
					ProcessInputsWatchers.Add(SimCallbackObject);
				}

				if (SimCallbackObject->HasOption(ESimCallbackOptions::PhysicsObjectUnregister))
				{
					PhysicsObjectUnregistrationWatchers.Add(SimCallbackObject);
				}
			}
		}

		//save any pending data for this particular interval
		{
			QUICK_SCOPE_CYCLE_COUNTER(Chaos_SetCurrentInput_Internal);
			for (const FSimCallbackInputAndObject& InputAndCallbackObj : PushData.SimCallbackInputs)
			{
				InputAndCallbackObj.CallbackObject->SetCurrentInput_Internal(InputAndCallbackObj.Input);
			}
		}

		//remove any callbacks that are unregistered
		{
			QUICK_SCOPE_CYCLE_COUNTER(Chaos_ProcessPushedData_RemoveCallbacks);
			for (ISimCallbackObject* RemovedCallbackObject : PushData.SimCallbackObjectsToRemove)
			{
				RemovedCallbackObject->bPendingDelete = true;

				if (RemovedCallbackObject->HasOption(ESimCallbackOptions::Rewind))
				{
					if (MRewindCallback)
					{
						MRewindCallback->UnregisterRewindableSimCallback_Internal(RemovedCallbackObject);
					}
				}
			}

			for (int32 Idx = SimCallbackObjects.Num() - 1; Idx >= 0; --Idx)
			{
				if (SimCallbackObjects[Idx]->bPendingDelete)
				{
					SimCallbackObjects.RemoveAtSwap(Idx, EAllowShrinking::No);
				}
			}

			for (int32 Idx = MidPhaseModifiers.Num() - 1; Idx >= 0; --Idx)
			{
				if (MidPhaseModifiers[Idx]->bPendingDelete)
				{
					//will also be in SimCallbackObjects so we'll delete it in that loop
					MidPhaseModifiers.RemoveAtSwap(Idx, EAllowShrinking::No);
				}
			}

			for (int32 Idx = CCDModifiers.Num() - 1; Idx >= 0; --Idx)
			{
				if (CCDModifiers[Idx]->bPendingDelete)
				{
					//will also be in SimCallbackObjects so we'll delete it in that loop
					CCDModifiers.RemoveAtSwap(Idx, EAllowShrinking::No);
				}
			}

			for (int32 Idx = StrainModifiers.Num() - 1; Idx >= 0; --Idx)
			{
				if (StrainModifiers[Idx]->bPendingDelete)
				{
					//will also be in SimCallbackObjects so we'll delete it in that loop
					StrainModifiers.RemoveAtSwap(Idx, EAllowShrinking::No);
				}
			}

			for (int32 Idx = ContactModifiers.Num() - 1; Idx >= 0; --Idx)
			{
				if (ContactModifiers[Idx]->bPendingDelete)
				{
					//will also be in SimCallbackObjects so we'll delete it in that loop
					ContactModifiers.RemoveAtSwap(Idx, EAllowShrinking::No);
				}
			}

			for (int32 Idx = RegistrationWatchers.Num() - 1; Idx >= 0; --Idx)
			{
				if (RegistrationWatchers[Idx]->bPendingDelete)
				{
					//will also be in SimCallbackObjects so we'll delete it in that loop
					RegistrationWatchers.RemoveAtSwap(Idx, EAllowShrinking::No);
				}
			}

			for (int32 Idx = UnregistrationWatchers.Num() - 1; Idx >= 0; --Idx)
			{
				if (UnregistrationWatchers[Idx]->bPendingDelete)
				{
					//will also be in SimCallbackObjects so we'll delete it in that loop
					UnregistrationWatchers.RemoveAtSwap(Idx, EAllowShrinking::No);
				}
			}

			for (int32 Idx = ProcessInputsWatchers.Num() - 1; Idx >= 0; --Idx)
			{
				if (ProcessInputsWatchers[Idx]->bPendingDelete)
				{
					ProcessInputsWatchers.RemoveAtSwap(Idx, EAllowShrinking::No);
				}
			}

			for (int32 Idx = PhysicsObjectUnregistrationWatchers.Num() - 1; Idx >= 0; --Idx)
			{
				if (PhysicsObjectUnregistrationWatchers[Idx]->bPendingDelete)
				{
					//will also be in SimCallbackObjects so we'll delete it in that loop
					PhysicsObjectUnregistrationWatchers.RemoveAtSwap(Idx, EAllowShrinking::No);
				}
			}
		}

		if (FRewindData* RewindData = GetRewindData())
		{
			RewindData->CacheCurrentDirtyData(FFrameAndPhase::PrePushData);
		}

		ProcessSinglePushedData_Internal(PushData);

		//run any commands passed in. These don't generate outputs and are a one off so just do them here
		//note: commands run before sim callbacks. This is important for sub-stepping since we want each sub-step to have a consistent view
		//so for example if the user deletes a floor surface, we want all sub-steps to see that in the same way
		//also note, the commands run after data is marshalled over. This is important because data marshalling ensures any GT property changes are seen by command
		//for example a particle may not be created until marshalling occurs, and then a command could explicitly modify something like a collision setting
		{
			QUICK_SCOPE_CYCLE_COUNTER(Chaos_ProcessPushedData_SimCommands);
			// If we are not resimulating, insert scheduled commands for this step into the current PushData to be executed
			if (!IsResimming())
			{
				TArray<FSimCallbackCommandObject*>& ScheduledSimCommands = MarshallingManager.GetSimCommandsScheduled_Internal();
				for (int32 Idx = ScheduledSimCommands.Num() - 1; Idx >= 0; Idx--)
				{
					FSimCallbackCommandObject* SimCallbackObject = ScheduledSimCommands[Idx];
					if (SimCallbackObject->ShouldExecute(CurrentFrame, /*bIsResim*/false))
					{
						// If this scheduled command should get processed this step, inject it into the start of PushData:SimCommands to preserve call order and to cache the SimCommand in rewind history via PushData
						PushData.SimCommands.Insert(SimCallbackObject, 0);
						ScheduledSimCommands.RemoveAt(Idx, EAllowShrinking::No);
					}
				}
			}

			for (FSimCallbackCommandObject*& SimCallbackObject : PushData.SimCommands)
			{
				if (!SimCallbackObject)
				{
					// A scheduled command delayed for a later frame will leave a nullptr in the array
					continue;
				}

				if (SimCallbackObject->ShouldExecute(CurrentFrame, IsResimming()))
				{
					SimCallbackObject->SetSimAndDeltaTime_Internal(GetSolverTime(), MLastDt);
					SimCallbackObject->PreSimulate_Internal();
				}
				else if (SimCallbackObject->ShouldDelay(CurrentFrame, IsResimming()))
				{
					// Cache command to run on a future frame and null the pointer in this PushData
					MarshallingManager.EnqueueCommandScheduled_Internal(SimCallbackObject);
					SimCallbackObject = nullptr; // Stop tracking the command in this PushData
				}
			}
		}

		for (ISimCallbackObject* SimCallbackObject : PushData.SimCallbackObjectsToAdd)
		{
			if (SimCallbackObject->HasOption(ESimCallbackOptions::RunOnFrozenGameThread) == bGameThreadFrozen)
			{
				SimCallbackObject->SetSimAndDeltaTime_Internal(GetSolverTime(), MLastDt);
				SimCallbackObject->PostInitialize_Internal();
			}
		}
	}

	void FPBDRigidsSolver::ProcessSkippedFramePushData_Internal(FPushPhysicsData& PushData, int32 SkippedFrame)
	{
		// Process callback additions to keep callback lists consistent
		{
			SimCallbackObjects.Reserve(SimCallbackObjects.Num() + PushData.SimCallbackObjectsToAdd.Num());
			for (ISimCallbackObject* SimCallbackObject : PushData.SimCallbackObjectsToAdd)
			{
				if (HasSimulationCallback(SimCallbackObject))
				{
					SimCallbackObjects.Add(SimCallbackObject);
				}

				if (SimCallbackObject->HasOption(ESimCallbackOptions::MidPhaseModification))
				{
					MidPhaseModifiers.Add(SimCallbackObject);
				}

				if (SimCallbackObject->HasOption(ESimCallbackOptions::CCDModification))
				{
					CCDModifiers.Add(SimCallbackObject);
				}

				if (SimCallbackObject->HasOption(ESimCallbackOptions::StrainModification))
				{
					StrainModifiers.Add(SimCallbackObject);
				}

				if (SimCallbackObject->HasOption(ESimCallbackOptions::ContactModification))
				{
					ContactModifiers.Add(SimCallbackObject);
				}

				if (SimCallbackObject->HasOption(ESimCallbackOptions::ParticleRegister))
				{
					RegistrationWatchers.Add(SimCallbackObject);
				}

				if (SimCallbackObject->HasOption(ESimCallbackOptions::ParticleUnregister))
				{
					UnregistrationWatchers.Add(SimCallbackObject);
				}

				if (SimCallbackObject->HasOption(ESimCallbackOptions::Rewind))
				{
					if (MRewindCallback)
					{
						MRewindCallback->RegisterRewindableSimCallback_Internal(SimCallbackObject);
					}
				}

				if (SimCallbackObject->HasOption(ESimCallbackOptions::ProcessInputsInternal))
				{
					ProcessInputsWatchers.Add(SimCallbackObject);
				}

				if (SimCallbackObject->HasOption(ESimCallbackOptions::PhysicsObjectUnregister))
				{
					PhysicsObjectUnregistrationWatchers.Add(SimCallbackObject);
				}
			}
		}

		// Process callback removals
		{
			for (ISimCallbackObject* RemovedCallbackObject : PushData.SimCallbackObjectsToRemove)
			{
				RemovedCallbackObject->bPendingDelete = true;

				if (RemovedCallbackObject->HasOption(ESimCallbackOptions::Rewind))
				{
					if (MRewindCallback)
					{
						MRewindCallback->UnregisterRewindableSimCallback_Internal(RemovedCallbackObject);
					}
				}
			}

			auto RemovePendingDelete = [](auto& Array)
			{
				for (int32 Idx = Array.Num() - 1; Idx >= 0; --Idx)
				{
					if (Array[Idx]->bPendingDelete)
					{
						Array.RemoveAtSwap(Idx, EAllowShrinking::No);
					}
				}
			};

			RemovePendingDelete(SimCallbackObjects);
			RemovePendingDelete(MidPhaseModifiers);
			RemovePendingDelete(CCDModifiers);
			RemovePendingDelete(StrainModifiers);
			RemovePendingDelete(ContactModifiers);
			RemovePendingDelete(RegistrationWatchers);
			RemovePendingDelete(UnregistrationWatchers);
			RemovePendingDelete(ProcessInputsWatchers);
			RemovePendingDelete(PhysicsObjectUnregistrationWatchers);
		}

		// Execute SimCommands for the skipped frame
		{
			for (FSimCallbackCommandObject*& SimCallbackObject : PushData.SimCommands)
			{
				if (!SimCallbackObject)
				{
					continue;
				}

				if (SimCallbackObject->ShouldExecute(SkippedFrame, /*bIsResim=*/true))
				{
					SimCallbackObject->SetSimAndDeltaTime_Internal(GetSolverTime(), MLastDt);
					SimCallbackObject->PreSimulate_Internal();
				}
			}
		}

		// PostInitialize newly added callbacks
		for (ISimCallbackObject* SimCallbackObject : PushData.SimCallbackObjectsToAdd)
		{
			if (SimCallbackObject->HasOption(ESimCallbackOptions::RunOnFrozenGameThread) == bGameThreadFrozen)
			{
				SimCallbackObject->SetSimAndDeltaTime_Internal(GetSolverTime(), MLastDt);
				SimCallbackObject->PostInitialize_Internal();
			}
		}
	}

	void FPBDRigidsSolver::ConditionalApplyRewind_Internal()
	{
		// Note: checking MRewindData->IsResim() can lead to recursion into this function on the last resim frame since the call to AdvanceSolver is what advances RewindData's internal frame
		if(!IsShuttingDown() && ShouldApplyRewindCallbacks() && !GetEvolution()->IsResimming())
		{
			const int32 LastStep = MRewindData->CurrentFrame() - 1;
			const int32 ResimStep = MRewindCallback->TriggerRewindIfNeeded_Internal(LastStep);
			const int32 NumResimSteps = MRewindData->CurrentFrame() - ResimStep;

			if (ResimStep < 0)
			{
				// Clear ResimFrame if no valid resim frame was found
				MRewindData->SetResimFrame(INDEX_NONE);
				GetEvolution()->GetIslandManager().ResetParticleResimFrame();
				return;
			}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			const bool bEnableNetworkPredictionDebug = IsNetworkPhysicsPredictionEnabled() && CanDebugNetworkPhysicsPrediction();
			if (bEnableNetworkPredictionDebug)
			{
				UE_LOGF(LogChaos, Log, "	-> Trying to Rewind Frame = %d | At Time = %f | Num Steps = %d | Resim Step = %d | Last Step = %d | Manager Size = %d", CurrentFrame,  MTime, NumResimSteps, ResimStep, LastStep, MarshallingManager.GetNumHistory_Internal());
				for(auto& Handle : GetParticles().GetNonDisabledDynamicView())
				{
					if(!Handle.IsSleeping())
					{
						UE_LOGF(LogChaos, Log, "Particle Dynamic At Position = %ls | Velocity = %ls | Quaternion = %ls | Omega = %ls",
							*Handle.GetX().ToString(), *Handle.GetV().ToString(), *Handle.GetR().ToString(), *Handle.GetW().ToString());
					}
					else
					{
						UE_LOGF(LogChaos, Log, "Particle Sleeping At Position = %ls | Velocity = %ls | Quaternion = %ls | Omega = %ls", 
							*Handle.GetX().ToString(), *Handle.GetV().ToString(), *Handle.GetR().ToString(), *Handle.GetW().ToString());
					}
				}
			}
#endif

			{
				QUICK_SCOPE_CYCLE_COUNTER(ChaosRewindAndResim);

				if (MRewindData->CanRewindToFrame(ResimStep))
				{
					if (MRewindData->AllowRewindToFrame(ResimStep))
					{
						SetIsResimming(true);
						CurrentFrame = ResimStep;

#if WITH_CHAOS_VISUAL_DEBUGGER
						const bool bIsCVDTracing = FChaosVisualDebuggerTrace::IsTracing();
						const bool bIsTracingResimDetails = bIsCVDTracing && CVDDC_ResimDetails->IsChannelEnabled();
						if (bIsCVDTracing)
						{
							CVD_TRACE_SOLVER_START_FRAME(FPhysicsSolverBase, *this);
						}
#endif
						MRewindData->RewindToFrame(ResimStep);

#if DEBUG_REWIND_DATA
						UE_LOGF(LogChaos, Warning, "CLIENT | PT | ConditionalApplyRewind_Internal | PERFORMING RESIMULATION | Resim From Frame = %d | Num Steps = %d | To Current Frame: %d", ResimStep, NumResimSteps, CurrentFrame);
#endif

						FPushPhysicsData* ConsumerData = MarshallingManager.GetConsumerData_Internal();
						TArray<FPushPhysicsData*> RecordedPushData = MarshallingManager.StealHistory_Internal(NumResimSteps);
						bool bFirst = true;

						FResimDebugInfo DebugInfo;
						FDurationTimer ResimTimer(DebugInfo.ResimTime);
						{
							QUICK_SCOPE_CYCLE_COUNTER(ChaosRewindAndResim_Resim);

							// Adaptive coalescing: when enabled, frames past the accurate head may be grouped and
							// simulated with a larger dt if the projected resim time exceeds the time budget.
							const bool bCoalescingEnabled = PhysicsReplicationCVars::ResimulationCVars::bEnableAdaptiveCoalescing;
							const int32 AccurateHeadEnd = FMath::Min(ResimStep + FMath::Max(PhysicsReplicationCVars::ResimulationCVars::AccurateHeadFrames, 1) - 1, LastStep);
							const int32 MaxCoalesceMultiplier = FMath::Max(PhysicsReplicationCVars::ResimulationCVars::MaxCoalesceMultiplier, 1);
							const uint64 ResimLoopStartCycles = FPlatformTime::Cycles64();
							int32 StepsDone = 0;

							// Do rollback as necessary
							const FReal FixedDt = GetAsyncDeltaTime();
							int32 Step = ResimStep;
							while (Step <= LastStep)
							{
								// Compute adaptive coalescing multiplier
								int32 CoalesceMultiplier = 1;
								if (bCoalescingEnabled && Step > AccurateHeadEnd && StepsDone > 0)
								{
									const uint64 ElapsedCycles = FPlatformTime::Cycles64() - ResimLoopStartCycles;
									const uint64 AvgCyclesPerStep = ElapsedCycles / static_cast<uint64>(StepsDone);
									const int32 RemainingSteps = LastStep - Step + 1;
									const uint64 ProjectedTotalCycles = ElapsedCycles + AvgCyclesPerStep * static_cast<uint64>(RemainingSteps);

									const uint64 BudgetTotalCycles = static_cast<uint64>(FixedDt * PhysicsReplicationCVars::ResimulationCVars::CoalescingBudgetMultiplier / FPlatformTime::GetSecondsPerCycle64());

									if (BudgetTotalCycles > 0 && ProjectedTotalCycles > BudgetTotalCycles)
									{
										CoalesceMultiplier = FMath::CeilToInt32(static_cast<double>(ProjectedTotalCycles) / static_cast<double>(BudgetTotalCycles));
										CoalesceMultiplier = FMath::Clamp(CoalesceMultiplier, 1, MaxCoalesceMultiplier);
										CoalesceMultiplier = FMath::Min(CoalesceMultiplier, LastStep - Step + 1);
									}
								}

								const int32 TargetStep = FMath::Min(Step + CoalesceMultiplier - 1, LastStep);
								const int32 ActualMultiplier = TargetStep - Step + 1;

								if (ActualMultiplier > 1)
								{
									DebugInfo.FramesCoalesced += ActualMultiplier - 1;
									DebugInfo.MaxMultiplierUsed = FMath::Max(DebugInfo.MaxMultiplierUsed, ActualMultiplier);
								}

								// Handle skipped frames: process callback lifecycle and SimCommands, advance counters.
								// MTime is temporarily advanced per skipped frame so callbacks see the correct GetSolverTime(),
								// then restored so AdvanceSolver can advance by the full coalesced dt without double-counting.
								const FReal SavedSolverTime = GetSolverTime();
								for (int32 Skip = Step; Skip < TargetStep; ++Skip)
								{
									if (!bFirst && !bUseCollisionResimCache)
									{
										MRewindData->StepNonResimParticles(Skip);
									}

									SetSolverTime(GetSolverTime() + FixedDt);

									if ((LastStep - Skip) < RecordedPushData.Num())
									{
										FPushPhysicsData* SkippedPD = RecordedPushData[LastStep - Skip];
										ProcessSkippedFramePushData_Internal(*SkippedPD, Skip);
										MarshallingManager.FreeDataToHistory_Internal(SkippedPD);
									}

									CurrentFrame++;
									MRewindData->SkipFrames(1);
								}
								SetSolverTime(SavedSolverTime);

								// Now CurrentFrame == TargetStep, process the target frame normally
#if WITH_CHAOS_VISUAL_DEBUGGER
								if (bIsCVDTracing && !bFirst)
								{
									CVD_TRACE_SOLVER_END_FRAME(FPhysicsSolverBase, *this);
									CVD_TRACE_SOLVER_START_FRAME(FPhysicsSolverBase, *this);
								}

								if (bIsTracingResimDetails)
								{
									CVD_SCOPE_TRACE_SOLVER_STEP(CVDDC_ResimDetails, TEXT("Resim Step Start"));
									CVD_TRACE_PARTICLES_SOA(GetEvolution()->GetParticles(), &GetEvolution()->GetRigidClustering());
								}
#endif

								if ((LastStep - TargetStep) < RecordedPushData.Num())
								{
									if (!bFirst && !bUseCollisionResimCache)
									{
										MRewindData->StepNonResimParticles(TargetStep);
									}

									if (PhysicsReplicationCVars::ResimulationCVars::bApplyTargetsWhileResimulating || bFirst)
									{
										// Update all the particles having received a target from the server
										MRewindData->ApplyTargets(TargetStep, bFirst);
									}

									FPushPhysicsData* PushData = RecordedPushData[LastStep - TargetStep];	//push data is sorted as latest first

									// Scale dt and AsyncDt for coalesced frames so the solver and any code
									// calling GetAsyncDeltaTime() see the actual dt being simulated.
									const FReal OriginalDt = PushData->ExternalDt;
									if (ActualMultiplier > 1)
									{
										PushData->ExternalDt *= ActualMultiplier;
										SetAsyncDeltaTime(PushData->ExternalDt);
									}

									if (bFirst)
									{
										SCOPED_NAMED_EVENT(ChaosRewindAndResim_ResetCollisions, FColor::Red);
										MTime = PushData->StartTime;	//not sure if sub-steps have proper StartTime so just do this once and let solver evolve remaining time

										GetEvolution()->ResetCollisions();
									}

									GetEvolution()->SetReset(bFirst);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
									if (bEnableNetworkPredictionDebug)
									{
										UE_LOGF(LogChaos, Log, "		-> Re-simulating Frame = %d (x%d dt)", TargetStep, ActualMultiplier);
										for (auto& Handle : GetParticles().GetNonDisabledDynamicView())
										{
											if (GetEvolution()->GetIslandManager().GetParticleIsland(Handle.Handle()))
											{
												if (!Handle.IsSleeping())
												{
													UE_LOGF(LogChaos, Log, "Particle Dynamic At Position = %ls | Velocity = %ls | Quaternion = %ls | Omega = %ls | Resim Frame = %d | Sync State = %d | Needs Resim = %d | Unique Idx = %d",
														*Handle.GetX().ToString(), *Handle.GetV().ToString(), *Handle.GetR().ToString(), *Handle.GetW().ToString(),
														GetEvolution()->GetIslandManager().GetParticleIsland(Handle.Handle())->GetResimFrame(), (uint8)Handle.SyncState(),
														GetEvolution()->GetIslandManager().GetParticleIsland(Handle.Handle())->NeedsResim(), Handle.UniqueIdx().Idx);
												}
												else
												{
													UE_LOGF(LogChaos, Log, "Particle Sleeping At Position = %ls | Velocity = %ls | Quaternion = %ls | Omega = %ls | Resim Frame = %d | Sync State = %d | Needs Resim = %d | Unique Idx = %d",
														*Handle.GetX().ToString(), *Handle.GetV().ToString(), *Handle.GetR().ToString(), *Handle.GetW().ToString(),
														GetEvolution()->GetIslandManager().GetParticleIsland(Handle.Handle())->GetResimFrame(), (uint8)Handle.SyncState(),
														GetEvolution()->GetIslandManager().GetParticleIsland(Handle.Handle())->NeedsResim(), Handle.UniqueIdx().Idx);
												}
											}
										}
									}
#endif
									{
										QUICK_SCOPE_CYCLE_COUNTER(ChaosRewindAndResim_RewindCallback_PreResimStep_Internal);
										MRewindCallback->PreResimStep_Internal(TargetStep, bFirst);
									}

									// Run the advance but omit the game thread callbacks as we're executing during the physics tick
									FSolverTasksPTOnly ImmediateTask(*this, PushData);
									//ensure(bSolverHasFrozenGameThreadCallbacks == false);	//We don't support this for resim as it's very expensive and difficult to schedule
									ImmediateTask.AdvanceSolver();
									{
										QUICK_SCOPE_CYCLE_COUNTER(ChaosRewindAndResim_RewindCallback_PostResimStep_Internal);
										MRewindCallback->PostResimStep_Internal(TargetStep);
									}

									// Restore dt state after the coalesced step. ExternalDt is written back in-place in
									// the history queue (safe because nothing reads it between FreeDataToHistory and here).
									// MLastDt and AsyncDt are reset to the base fixed dt for the next iteration.
									if (ActualMultiplier > 1)
									{
										PushData->ExternalDt = OriginalDt;
										SetAsyncDeltaTime(FixedDt);
									}
									MLastDt = FixedDt;

									bFirst = false;
								}

								Step = TargetStep + 1;
								StepsDone++;
							}
						}

						// Clean up after resim. This is where particles are unfrozen if they were frozen due to bubble resim.
						MRewindData->EndResim();

#if WITH_CHAOS_VISUAL_DEBUGGER
						if (bIsCVDTracing)
						{
							CVD_TRACE_SOLVER_END_FRAME(FPhysicsSolverBase, *this);
						}
#endif

						MarshallingManager.SetConsumerData_Internal(ConsumerData);
						GetEvolution()->GetIslandManager().ResetParticleResimFrame();

						SetIsResimming(false);
						GetEvolution()->SetReset(false);

						ResimTimer.Stop();

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
						if (DebugInfo.FramesCoalesced > 0)
						{
							UE_LOG(LogChaos, Log, TEXT("Resim adaptive coalescing: %d frames coalesced, max multiplier %dx, resim time %.3fms"),
								DebugInfo.FramesCoalesced, DebugInfo.MaxMultiplierUsed, DebugInfo.ResimTime * 1000.0);
						}
#endif

						MRewindCallback->SetResimDebugInfo_Internal(DebugInfo);
					}
				}
#if DEBUG_REWIND_DATA
				else // !CanRewindToFrame
				{
					UE_LOGF(LogChaos, Log, "CLIENT | PT | ConditionalApplyRewind_Internal | Resimulation failed, FRewindData::CanRewindToFrame returned false | Current Frame = %d | Num Steps = %d | Resim Frame = %d | Last Frame = %d | Rewind History Size = %d", CurrentFrame, NumResimSteps, ResimStep, LastStep, MarshallingManager.GetNumHistory_Internal());
				}
#endif
			}
		}
	}

	void FPBDRigidsSolver::SetIsResimming(bool bIsResimming)
	{
		GetEvolution()->SetResim(bIsResimming);

#if WITH_CHAOS_VISUAL_DEBUGGER
		EChaosVDContextAttributes Attributes = static_cast<EChaosVDContextAttributes>(GetChaosVDContextData().Attributes);
		if (bIsResimming)
		{
			EnumAddFlags(Attributes,  EChaosVDContextAttributes::Resimulated);
		}
		else
		{
			EnumRemoveFlags(Attributes,  EChaosVDContextAttributes::Resimulated);
		}

		GetChaosVDContextData().Attributes = static_cast<int32>(Attributes);
#endif	
	}

	void FPBDRigidsSolver::CompleteSceneSimulation()
	{
		LLM_SCOPE(ELLMTag::Chaos);
		SCOPE_CYCLE_COUNTER(STAT_BufferPhysicsResults);
		if (!IsResimming() || (GetRewindData() && GetRewindData()->IsFinalResim()))
		{
			EventPreBuffer.Broadcast(MLastDt);
			GetDirtyParticlesBuffer()->CaptureSolverData(this);
			BufferPhysicsResults();
		}
	}


	template <typename LambdaType>
	int32 FPBDRigidsSolver::DispatchTasksBufferPhysicsResults(int32 DispatchBatchIndex, int32 NumDirty, LambdaType ReturnHandle)
	{
		using TSOA = typename TParticleView<FPBDRigidParticles>::TSOA;
		using THandle = typename TSOA::THandleType;
		using THandleBase = typename THandle::THandleBase;

		const bool bIsResim = GetEvolution()->IsResimming();

		// For performance reason collect on one threads the rigid proxies and on another one the geometry collection and the cluster union.
		// That way buffering results from rigid proxy could start after the first batch of rigid proxy collected, while geometry collection, cluster union 
		// and other batches of rigid cluster will be still collected.
		auto CollectRigidResults = [bIsResim, ReturnHandle](int32 StartIndex, int32 EndIndex, TArray<FSingleParticlePhysicsProxy*>& ActiveRigid)
		{
			for (int32 Idx = StartIndex; Idx < EndIndex; ++Idx)
			{
				THandleBase Handle = ReturnHandle(Idx);
				FTransientPBDRigidParticleHandle& DirtyParticle = static_cast<FTransientPBDRigidParticleHandle&>(Handle);

				const EParticleType ParticleType = DirtyParticle.GetParticleType();
				if ((ParticleType == EParticleType::Kinematic) || (ParticleType == EParticleType::Static))
				{
					ensure(false);
				}

				if (IPhysicsProxyBase* Proxy = DirtyParticle.Handle()->PhysicsProxy())
				{
					switch (Proxy->GetType())
					{
					case EPhysicsProxyType::GeometryCollectionType:
					case EPhysicsProxyType::ClusterUnionProxy:
					{
						if (auto ClusterParticle = DirtyParticle.CastToClustered())
						{
							if (ClusterParticle->InternalCluster())
							{

								const bool bActivateChildren =
									ClusterParticle->ObjectState() == EObjectStateType::Dynamic ||
									Proxy->GetType() != EPhysicsProxyType::ClusterUnionProxy;

								// If the cluster is dynamic, all children will likely need a transform update
								// otherwise we should not need a full update (if the cluster is still we only
								// need to consider de-clustered objects which should be in other parts of the 
								// dirty view)
								if (bActivateChildren)
								{
									const TSet<IPhysicsProxyBase*> Proxies = ClusterParticle->PhysicsProxies();
									for (IPhysicsProxyBase* ClusterProxy : Proxies)
									{
										if (!ClusterProxy)
										{
											continue;
										}
										switch (ClusterProxy->GetType())
										{
										case EPhysicsProxyType::SingleParticleProxy:
											ActiveRigid.Add((FSingleParticlePhysicsProxy*)ClusterProxy);
											break;
										case EPhysicsProxyType::GeometryCollectionType:
											break;
										default:
											ensure(false);
											break;
										}
									}
								}
							}
							else
							{
								switch (Proxy->GetType())
								{
								case EPhysicsProxyType::SingleParticleProxy:
									ActiveRigid.Add((FSingleParticlePhysicsProxy*)Proxy);
									break;
								case EPhysicsProxyType::GeometryCollectionType:
									break;
								default:
									ensure(false);
									break;
								}
							}
						}
						else
						{
							ensure(false);
						}
					}
					break;
					case EPhysicsProxyType::SingleParticleProxy:
					{
						if (!bIsResim || DirtyParticle.SyncState() == ESyncState::HardDesync)
						{
							// Although per-particle we can control the syncing of target positions (see ShouldUpdateFromSimulation)
							// we cannot avoid marking kinematics as dirty (unless the global config Chaos::SyncKinematicOnGameThread
							// forces it) because we always need the correct velocities/dynamics to be synced back from the kinematic
							// target.
							// FSingleParticlePhysicsProxy ::PullFromPhysicsState will use the proper checks to see whether it needs
							// to sync the particle positions when the dirty particle is processed.
							if (!(Chaos::SyncKinematicOnGameThread == 0 && DirtyParticle.ObjectState() == EObjectStateType::Kinematic))
							{
								ActiveRigid.Add((FSingleParticlePhysicsProxy*)Proxy);
							}
						}
					}
					break;

					default:
						check(false);
						break;
					}
				}
			}
		};

		auto CollectGCAndClustersResults = [bIsResim, ReturnHandle](int32 StartIndex, int32 EndIndex,
			TArray<FGeometryCollectionPhysicsProxy*>& ActiveGC,
			TArray<FClusterUnionPhysicsProxy*>& ActiveClusterUnions)
			{
				for (int32 Idx = StartIndex; Idx < EndIndex; ++Idx)
				{
					THandleBase Handle = ReturnHandle(Idx);
					FTransientPBDRigidParticleHandle& DirtyParticle = static_cast<FTransientPBDRigidParticleHandle&>(Handle);

					const EParticleType ParticleType = DirtyParticle.GetParticleType();
					if ((ParticleType == EParticleType::Kinematic) || (ParticleType == EParticleType::Static))
					{
						ensure(false);
					}

					if (IPhysicsProxyBase* Proxy = DirtyParticle.Handle()->PhysicsProxy())
					{
						switch (Proxy->GetType())
						{
						case EPhysicsProxyType::GeometryCollectionType:
						case EPhysicsProxyType::ClusterUnionProxy:
						{
							if (auto ClusterParticle = DirtyParticle.CastToClustered())
							{
								if (ClusterParticle->InternalCluster())
								{
									if (Proxy->GetType() == EPhysicsProxyType::ClusterUnionProxy)
									{
										ActiveClusterUnions.AddUnique((FClusterUnionPhysicsProxy*)(Proxy));
									}

									const bool bActivateChildren =
										ClusterParticle->ObjectState() == EObjectStateType::Dynamic ||
										Proxy->GetType() != EPhysicsProxyType::ClusterUnionProxy;

									// If the cluster is dynamic, all children will likely need a transform update
									// otherwise we should not need a full update (if the cluster is still we only
									// need to consider de-clustered objects which should be in other parts of the 
									// dirty view)
									if (bActivateChildren)
									{
										const TSet<IPhysicsProxyBase*> Proxies = ClusterParticle->PhysicsProxies();
										for (IPhysicsProxyBase* ClusterProxy : Proxies)
										{
											if (!ClusterProxy)
											{
												continue;
											}

											switch (ClusterProxy->GetType())
											{
											case EPhysicsProxyType::SingleParticleProxy:
												break;
											case EPhysicsProxyType::GeometryCollectionType:
												ActiveGC.AddUnique((FGeometryCollectionPhysicsProxy*)(ClusterProxy));
												break;
											default:
												ensure(false);
												break;
											}
										}
									}
								}
								else
								{
									switch (Proxy->GetType())
									{
									case EPhysicsProxyType::SingleParticleProxy:
										break;
									case EPhysicsProxyType::GeometryCollectionType:
										ActiveGC.AddUnique((FGeometryCollectionPhysicsProxy*)(Proxy));
										break;
									default:
										ensure(false);
										break;
									}
								}
							}
							else
							{
								ensure(false);
							}
						}
						break;
						case EPhysicsProxyType::SingleParticleProxy:
							break;

						default:
							check(false);
							break;
						}
					}
				}
			};

		return TaskDispatcher->CollectPhysicsResults(CollectRigidResults, CollectGCAndClustersResults, NumDirty, DispatchBatchIndex);
	}

	void FPBDRigidsSolver::BufferPhysicsResultsSingleThreaded(TArray<FSingleParticlePhysicsProxy*>& ActiveRigid, TArray<FGeometryCollectionPhysicsProxy*>& ActiveGC, TArray<FClusterUnionPhysicsProxy*>& ActiveClusterUnions)
	{
		ActiveGC.Reserve(GeometryCollectionPhysicsProxies_Internal.Num());

		FPullPhysicsData* PullData = MarshallingManager.GetCurrentPullData_Internal();

		TParticleView<FPBDRigidParticles>& DirtyParticles = GetParticles().GetDirtyParticlesView();
		const int32 NumDirty = DirtyParticles.Num();

		// We can have at most NumDirty overall active particles across all dirty proxy lists
		// and in each list at most NumProxies. So we take the min, otherwise for example in a world
		// with a few million static particles and one dynamic we'll allocate enough storage for the
		// millions of particles despite only ever syncing one
		
		ActiveRigid.Reserve(FMath::Min(SingleParticlePhysicsProxies_PT.Num(), NumDirty));

		ActiveClusterUnions.Reserve(FMath::Min(ClusterUnionPhysicsProxies_Internal.Num(), NumDirty));

		const bool bIsResim = GetEvolution()->IsResimming();

		{
			ensure(PullData->DirtyRigids.Num() == 0);	//we only fill this once per frame
			int32 BufferIdx = 0;

			for (TPBDRigidParticleHandleImp<FReal, 3, false>& DirtyParticle : DirtyParticles)
			{
				const EParticleType ParticleType = DirtyParticle.GetParticleType();
				if ((ParticleType == EParticleType::Kinematic) || (ParticleType == EParticleType::Static))
				{
					ensure(false);
				}

				if (IPhysicsProxyBase* Proxy = DirtyParticle.Handle()->PhysicsProxy())
				{
					switch (Proxy->GetType())
					{
					case EPhysicsProxyType::GeometryCollectionType:
					case EPhysicsProxyType::ClusterUnionProxy:
					{
						if (auto ClusterParticle = DirtyParticle.CastToClustered())
						{
							if (ClusterParticle->InternalCluster())
							{
								if (Proxy->GetType() == EPhysicsProxyType::ClusterUnionProxy)
								{
									ActiveClusterUnions.AddUnique((FClusterUnionPhysicsProxy*)(Proxy));
								}

								const bool bActivateChildren =
									ClusterParticle->ObjectState() == EObjectStateType::Dynamic ||
									Proxy->GetType() != EPhysicsProxyType::ClusterUnionProxy;

								// If the cluster is dynamic, all children will likely need a transform update
								// otherwise we should not need a full update (if the cluster is still we only
								// need to consider de-clustered objects which should be in other parts of the 
								// dirty view)
								if (bActivateChildren)
								{
									const TSet<IPhysicsProxyBase*> Proxies = ClusterParticle->PhysicsProxies();
									for (IPhysicsProxyBase* ClusterProxy : Proxies)
									{
										if (!ClusterProxy)
										{
											continue;
										}

										switch (ClusterProxy->GetType())
										{
										case EPhysicsProxyType::SingleParticleProxy:
											ActiveRigid.AddUnique((FSingleParticlePhysicsProxy*)ClusterProxy);
											break;
										case EPhysicsProxyType::GeometryCollectionType:
											ActiveGC.AddUnique((FGeometryCollectionPhysicsProxy*)(ClusterProxy));
											break;
										default:
											ensure(false);
											break;
										}
									}
								}
							}
							else
							{
								switch (Proxy->GetType())
								{
								case EPhysicsProxyType::SingleParticleProxy:
									ActiveRigid.AddUnique((FSingleParticlePhysicsProxy*)Proxy);
									break;
								case EPhysicsProxyType::GeometryCollectionType:
									ActiveGC.AddUnique((FGeometryCollectionPhysicsProxy*)(Proxy));
									break;
								default:
									ensure(false);
									break;
								}
							}
						}
						else
						{
							ensure(false);
						}
					}
					break;
					case EPhysicsProxyType::SingleParticleProxy:
					{
						if (!bIsResim || DirtyParticle.SyncState() == ESyncState::HardDesync)
						{
							// Although per-particle we can control the syncing of target positions (see ShouldUpdateFromSimulation)
							// we cannot avoid marking kinematics as dirty (unless the global config Chaos::SyncKinematicOnGameThread
							// forces it) because we always need the correct velocities/dynamics to be synced back from the kinematic
							// target.
							// FSingleParticlePhysicsProxy::PullFromPhysicsState will use the proper checks to see whether it needs
							// to sync the particle positions when the dirty particle is processed.
							if (!(Chaos::SyncKinematicOnGameThread == 0 && DirtyParticle.ObjectState() == EObjectStateType::Kinematic))
							{
								ActiveRigid.AddUnique((FSingleParticlePhysicsProxy*)Proxy);
							}
						}
					}
					break;

					default:
						check(false);
						break;
					}

				}
			}
		}
	}

	void FPBDRigidsSolver::BufferPhysicsResults()
	{
		FPullPhysicsData* PullData = MarshallingManager.GetCurrentPullData_Internal();

		const bool bRunMultithreading = ShouldExecuteTasks(GetParticles().GetDirtyParticlesView().Num()) && !CVars::bSingleThreadPullData;
		if (bRunMultithreading)
		{
			using TSOA = typename TParticleView<FPBDRigidParticles>::TSOA;
			using THandle = typename TSOA::THandleType;
			using THandleBase = typename THandle::THandleBase;

			TParticleView<FPBDRigidParticles>& DirtyParticles = GetParticles().GetDirtyParticlesView();
			const int32 NumDirty = DirtyParticles.Num();
			int32 DispatchBatchIndex = 0;
			TaskDispatcher->Initialize(DirtyParticles);

			for (int32 ViewIndex = 0; ViewIndex < DirtyParticles.SOAViews.Num(); ++ViewIndex)
			{
				const TSOAView<TSOA>& SOAView = DirtyParticles.SOAViews[ViewIndex];
				const int32 NumParticles = SOAView.Size();
				if (NumParticles == 0)
				{
					continue;
				}
				// Dispatch tasks according if it is an array of handles or a SOA
				if (const TArray<THandle*>* CurHandlesArray = SOAView.HandlesArray)
				{
					auto ReturnHandle = [CurHandlesArray](int32 Index) -> TParticleView<FPBDRigidParticles>::THandle::THandleBase
					{
						THandle* HandlePtr = (*CurHandlesArray)[Index];
						return THandleBase(HandlePtr->GeometryParticles, HandlePtr->ParticleIdx);
					};
					using LambdaType = decltype(ReturnHandle);
					const int32 NumHandles = CurHandlesArray->Num();
					check(NumHandles == NumParticles);
					DispatchBatchIndex += DispatchTasksBufferPhysicsResults<LambdaType>(DispatchBatchIndex, NumHandles, ReturnHandle);
				}
				else if (TSOA* Soa = SOAView.SOA)
				{
					auto ReturnHandle = [Soa](int32 Index) { return THandleBase(Soa, Index); };
					using LambdaType = decltype(ReturnHandle);
					DispatchBatchIndex += DispatchTasksBufferPhysicsResults<LambdaType>(DispatchBatchIndex, NumParticles, ReturnHandle);
				}
			}

			//we only fill this once per frame
			ensure(PullData->DirtyRigids.Num() == 0);
			ensure(PullData->DirtyGeometryCollections.Num() == 0);
			ensure(PullData->DirtyClusterUnions.IsEmpty());
			check(TaskDispatcher->GetNumRigidBatches() == DispatchBatchIndex);
			TaskDispatcher->BufferRigidResults(PullData);
			TaskDispatcher->BufferGCResults(PullData, this);
			TaskDispatcher->BufferClusterResults(PullData);
		}
		else
		{
			// Those arrays are here only for the single threaded version
			TArray<FSingleParticlePhysicsProxy*> ActiveRigid;
			TArray<FGeometryCollectionPhysicsProxy*> ActiveGC;
			TArray<FClusterUnionPhysicsProxy*> ActiveClusterUnions;
			BufferPhysicsResultsSingleThreaded(ActiveRigid, ActiveGC, ActiveClusterUnions);
			{
				//we only fill this once per frame
				ensure(PullData->DirtyRigids.Num() == 0);
				PullData->DirtyRigids.Reserve(ActiveRigid.Num());

				for (int32 Idx = 0; Idx < ActiveRigid.Num(); ++Idx)
				{
					PullData->DirtyRigids.AddDefaulted();
					ActiveRigid[Idx]->BufferPhysicsResults(PullData->DirtyRigids.Last());
				}
			}
			{
				ensure(PullData->DirtyGeometryCollections.Num() == 0);	//we only fill this once per frame
				PullData->DirtyGeometryCollections.Reserve(ActiveGC.Num());

				for (int32 Idx = 0; Idx < ActiveGC.Num(); ++Idx)
				{
					PullData->DirtyGeometryCollections.AddDefaulted();
					ActiveGC[Idx]->BufferPhysicsResults_Internal(this, PullData->DirtyGeometryCollections.Last());
					ActiveGC[Idx]->FlushDirtyParticles(*this);
				}
			}
			{
				ensure(PullData->DirtyClusterUnions.IsEmpty());
				PullData->DirtyClusterUnions.Reserve(ActiveClusterUnions.Num());

				for (int32 Idx = 0; Idx < ActiveClusterUnions.Num(); ++Idx)
				{
					PullData->DirtyClusterUnions.AddDefaulted();
					ActiveClusterUnions[Idx]->BufferPhysicsResults_Internal(PullData->DirtyClusterUnions.Last());
				}
			}
		}

		{
			ensure(PullData->DirtyJointConstraints.Num() == 0);	//we only fill this once per frame
			PullData->DirtyJointConstraints.Reserve(JointConstraintPhysicsProxies_Internal.Num());

			for(int32 Idx = 0; Idx < JointConstraintPhysicsProxies_Internal.Num(); ++Idx)
			{
				PullData->DirtyJointConstraints.AddDefaulted();
				JointConstraintPhysicsProxies_Internal[Idx]->BufferPhysicsResults(PullData->DirtyJointConstraints.Last());
			}
		}

		{
			ensure(PullData->DirtyCharacterGroundConstraints.Num() == 0);	//we only fill this once per frame
			PullData->DirtyCharacterGroundConstraints.Reserve(CharacterGroundConstraintProxies_Internal.Num());

			for (int32 Idx = 0; Idx < CharacterGroundConstraintProxies_Internal.Num(); ++Idx)
			{
				PullData->DirtyCharacterGroundConstraints.AddDefaulted();
				CharacterGroundConstraintProxies_Internal[Idx]->BufferPhysicsResults(PullData->DirtyCharacterGroundConstraints.Last());
			}
		}

		// Calculate and move post-resim error correction data from RewindData to FPullPhysicsData for marshaling
		if (FRewindData* RewindData = GetRewindData())
		{
			RewindData->BufferPhysicsResults(PullData->DirtyRigidErrors);
		}

		if (bRunMultithreading)
		{
			TaskDispatcher->WaitTaskEndBufferResults();
			for (int32 Idx = 0; Idx < TaskDispatcher->GetActiveGC().Num(); ++Idx)
			{
				TaskDispatcher->GetActiveGC()[Idx]->FlushDirtyParticles(*this);
			}
		}

		// Now that results have been buffered we have completed a solve step so we can broadcast that event
		EventPostSolve.Broadcast(MLastDt);

		Particles.ClearTransientDirty();
	}

	void FPBDRigidsSolver::BeginDestroy()
	{
		MEvolution->SetCanStartAsyncTasks(false);
	}
	
	// This function is not called during normal Engine execution.  
	// FPhysScene_ChaosInterface::EndFrame() calls 
	// FPhysScene_ChaosInterface::SyncBodies() instead, and then immediately afterwards 
	// calls FPBDRigidsSovler::SyncEvents_GameThread().  This function is used by tests,
	// however.
	void FPBDRigidsSolver::UpdateGameThreadStructures()
	{
		struct FDispatcher {} Dispatcher;
		PullPhysicsStateForEachDirtyProxy_External(Dispatcher);
	}

	int32 FPBDRigidsSolver::NumJointConstraints() const
	{
		return MEvolution->GetJointCombinedConstraints().NumConstraints();
	}
	
	int32 FPBDRigidsSolver::NumCollisionConstraints() const
	{
		return GetEvolution()->GetCollisionConstraints().NumConstraints();
	}

	void FPBDRigidsSolver::ResetStatCounters()
	{
		TRACE_COUNTER_SET(ChaosTraceCounter_MidPhase_NumShapePair, 0);
		TRACE_COUNTER_SET(ChaosTraceCounter_MidPhase_NumGeneric, 0);
	}

#ifndef CHAOS_COUNTER_STAT
#define CHAOS_COUNTER_STAT(Name, Value)\
SET_DWORD_STAT(STAT_ChaosCounter_##Name, Value); \
CSV_CUSTOM_STAT(PhysicsCounters, Name, Value, ECsvCustomStatOp::Set); \
TRACE_COUNTER_SET(ChaosTraceCounter_##Name, Value)
#endif

	void FPBDRigidsSolver::UpdateStatCounters() const
	{
		const int32 NumStatic = GetEvolution()->GetParticles().GetActiveStaticParticlesView().Num();
		const int32 NumKinematic = GetEvolution()->GetParticles().GetActiveKinematicParticlesView().Num();
		const int32 NumDynamic = GetEvolution()->GetParticles().GetNonDisabledDynamicView().Num();
		const int32 NumMoving = GetEvolution()->GetParticles().GetActiveDynamicMovingKinematicParticlesView().Num();

		// Particle counts
		CHAOS_COUNTER_STAT(NumDisabledBodies, GetEvolution()->GetParticles().GetAllParticlesView().Num() - GetEvolution()->GetParticles().GetNonDisabledView().Num());
		CHAOS_COUNTER_STAT(NumBodies, GetEvolution()->GetParticles().GetNonDisabledView().Num());
		CHAOS_COUNTER_STAT(NumDynamicBodies, NumDynamic);
		CHAOS_COUNTER_STAT(NumKinematicBodies, NumKinematic);
		CHAOS_COUNTER_STAT(NumStaticBodies, NumStatic);
		CHAOS_COUNTER_STAT(NumMovingBodies, NumMoving);
		CHAOS_COUNTER_STAT(NumGeometryCollectionBodies, (int32)GetEvolution()->GetParticles().GetGeometryCollectionParticles().Size());

		// Constraint counts
		CHAOS_COUNTER_STAT(NumIslands, GetEvolution()->GetIslandManager().GetNumIslands());
		CHAOS_COUNTER_STAT(NumIslandGroups, GetEvolution()->GetIslandGroupManager().GetNumActiveGroups());
		CHAOS_COUNTER_STAT(NumContacts, NumCollisionConstraints());
		CHAOS_COUNTER_STAT(NumJoints, NumJointConstraints());
		CHAOS_COUNTER_STAT(NumCharacterGroundConstraints, GetEvolution()->GetCharacterGroundConstraints().GetNumConstraints());

#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
		UpdateExpensiveStatCounters();
#endif

		// Collision detection info
		CHAOS_COUNTER_STAT(NumBroadPhasePairs, GetEvolution()->GetBroadPhase().GetNumBroadPhasePairs());
		CHAOS_COUNTER_STAT(NumMidPhases, GetEvolution()->GetBroadPhase().GetNumMidPhases());

		// Iterations
		CHAOS_COUNTER_STAT(NumPositionIterations, GetEvolution()->GetNumPositionIterations());
		CHAOS_COUNTER_STAT(NumVelocityIterations, GetEvolution()->GetNumVelocityIterations());
		CHAOS_COUNTER_STAT(NumProjectionIterations, GetEvolution()->GetNumProjectionIterations());

		// Solver parallelization
		if (CVars::Chaos_SolverPartitionManager_Partitioning == 1) // only update if graph coloring solver enabled
		{
			const int32 CollisionContainerId = GetEvolution()->GetCollisionConstraints().GetContainerId();
			const int32 LinearJointContainerId = GetEvolution()->GetJointCombinedConstraints().LinearConstraints.GetContainerId();
			const int32 NonLinearJointContainerId = GetEvolution()->GetJointCombinedConstraints().NonLinearConstraints.GetContainerId();
			const int32 SuspensionContainerId = GetEvolution()->GetSuspensionConstraints().GetContainerId();
			const int32 CharacterGroundContainerId = GetEvolution()->GetCharacterGroundConstraints().GetContainerId();

			const Private::FPBDIslandGroupManager& IslandGroupManager = GetEvolution()->GetIslandGroupManager();
			CHAOS_COUNTER_STAT(NumCollisionColors, IslandGroupManager.GetMaxSerialConstraintBatches(CollisionContainerId));
			CHAOS_COUNTER_STAT(NumLinearJointColors, IslandGroupManager.GetMaxSerialConstraintBatches(LinearJointContainerId));
			CHAOS_COUNTER_STAT(NumNonLinearJointColors, IslandGroupManager.GetMaxSerialConstraintBatches(NonLinearJointContainerId));
			CHAOS_COUNTER_STAT(NumSuspensionConstraintColors, IslandGroupManager.GetMaxSerialConstraintBatches(SuspensionContainerId));
			CHAOS_COUNTER_STAT(NumCharacterGroundConstraintColors, IslandGroupManager.GetMaxSerialConstraintBatches(CharacterGroundContainerId));
		}
		else
		{
			CHAOS_COUNTER_STAT(NumCollisionColors, 0);
			CHAOS_COUNTER_STAT(NumLinearJointColors, 0);
			CHAOS_COUNTER_STAT(NumNonLinearJointColors, 0);
			CHAOS_COUNTER_STAT(NumSuspensionConstraintColors, 0);
			CHAOS_COUNTER_STAT(NumCharacterGroundConstraintColors, 0);
		}
	}

	void FPBDRigidsSolver::UpdateExpensiveStatCounters() const
	{
		int32 NumValidCollisions = 0;
		int32 NumActiveCollisions = 0;
		int32 NumRestoredCollisions = 0;
		int32 NumManifoldPoints = 0;
		int32 NumActiveManifoldPoints = 0;
		int32 NumRestoredManifoldPoints = 0;
		int32 NumUpdatedManifoldPoints = 0;
		for (const FPBDCollisionConstraintHandle* Collision : GetEvolution()->GetCollisionConstraints().GetConstraints())
		{
			if (Collision == nullptr)
			{
				continue;
			}

			const FPBDCollisionConstraint& Contact = Collision->GetContact();
			if (Contact.IsEnabled())
			{
				if (Contact.GetManifoldPoints().Num() > 0)
				{
					++NumValidCollisions;
				}
				if (!Contact.AccumulatedImpulse.IsNearlyZero())
				{
					++NumActiveCollisions;
				}
				if (Contact.WasManifoldRestored())
				{
					++NumRestoredCollisions;
				}
				for (int32 PointIndex = 0; PointIndex < Contact.NumManifoldPoints(); ++PointIndex)
				{
					const FManifoldPoint& ManifoldPoint = Contact.GetManifoldPoint(PointIndex);
					const FManifoldPointResult& ManifoldPointResult = Contact.GetManifoldPointResult(PointIndex);
					++NumManifoldPoints;
					if (ManifoldPoint.Flags.bWasRestored || Contact.WasManifoldRestored())
					{
						++NumRestoredManifoldPoints;
					}
					if (ManifoldPoint.Flags.bWasReplaced)
					{
						++NumUpdatedManifoldPoints;
					}
					if (ManifoldPointResult.bIsValid)
					{
						if (!ManifoldPointResult.NetPushOut.IsNearlyZero())
						{
							++NumActiveManifoldPoints;
						}
					}
				}
			}
		}

		int32 NumStaticShapes = 0;
		int32 NumKinematicShapes = 0;
		int32 NumDynamicShapes = 0;
		for (const FTransientPBDRigidParticleHandle& Particle : Particles.GetActiveParticlesView())
		{
			const FConstGenericParticleHandle P = Particle.Handle();
			if (Particle.GetGeometry() != nullptr)
			{
				int32 NumShapes = 1;
				if (const FImplicitObjectUnion* Union = Particle.GetGeometry()->AsA<FImplicitObjectUnion>())
				{
					NumShapes = Union->GetNumLeafObjects();
				}
				if(auto* ClusteredParticle = Particle.CastToClustered())
				{
					if((ClusteredParticle->ConvexOptimizer().Get() != nullptr) && ClusteredParticle->ConvexOptimizer()->IsValid())
					{
						NumShapes = ClusteredParticle->ConvexOptimizer()->NumCollisionObjects();
					}
				}
				if (P->IsDynamic())
				{
					NumDynamicShapes += NumShapes;
				}
				else if (P->IsKinematic())
				{
					NumKinematicShapes += NumShapes;
				}
				else
				{
					NumStaticShapes += NumShapes;
				}
			}
		}

		CHAOS_COUNTER_STAT(NumValidConstraints, NumValidCollisions);
		CHAOS_COUNTER_STAT(NumActiveConstraints, NumActiveCollisions);
		CHAOS_COUNTER_STAT(NumRestoredConstraints, NumRestoredCollisions);
		CHAOS_COUNTER_STAT(NumManifoldPoints, NumManifoldPoints);
		CHAOS_COUNTER_STAT(NumActiveManifoldPoints, NumActiveManifoldPoints);
		CHAOS_COUNTER_STAT(NumRestoredManifoldPoints, NumRestoredManifoldPoints);
		CHAOS_COUNTER_STAT(NumUpdatedManifoldPoints, NumUpdatedManifoldPoints);
		CHAOS_COUNTER_STAT(NumStaticShapes, NumStaticShapes);
		CHAOS_COUNTER_STAT(NumKinematicShapes, NumKinematicShapes);
		CHAOS_COUNTER_STAT(NumDynamicShapes, NumDynamicShapes);
	}

	void FPBDRigidsSolver::DebugDrawShapes(const bool bShowStatic, const bool bShowKinematic, const bool bShowDynamic) const
	{
#if CHAOS_DEBUG_DRAW
		if (bShowStatic)
		{
			DebugDraw::DrawParticleShapes(FRigidTransform3(), Particles.GetActiveStaticParticlesView(), 1.0f, &ChaosSolverDebugDebugDrawSettings);
		}
		if (bShowKinematic)
		{
			DebugDraw::DrawParticleShapes(FRigidTransform3(), Particles.GetActiveKinematicParticlesView(), 1.0f, &ChaosSolverDebugDebugDrawSettings);
		}
		if (bShowDynamic)
		{
			DebugDraw::DrawParticleShapes(FRigidTransform3(), Particles.GetNonDisabledDynamicView(), 1.0f, &ChaosSolverDebugDebugDrawSettings);
		}
#endif
	}

	void FPBDRigidsSolver::PreIntegrateDebugDraw(FReal Dt) const
	{
#if CHAOS_DEBUG_DRAW
		QUICK_SCOPE_CYCLE_COUNTER(SolverDebugDraw);

		if (ChaosSolverDebugDrawPreIntegrationShapes == 1)
		{
			if (bChaosDebugDraw_UseNewQueue)
			{
				for (FTransientGeometryParticleHandle& Particle : Particles.GetActiveKinematicParticlesView())
				{
					FChaosDDParticle::DrawShapes(Particle.Handle(), FColor::Black);
				}
				for (FTransientGeometryParticleHandle& Particle : Particles.GetNonDisabledDynamicView())
				{
					FChaosDDParticle::DrawShapes(Particle.Handle(), FColor::Black);
				}
			}
			else
			{
				ChaosSolverDebugDebugDrawSettings.ShapesColorsPerState = DebugDraw::GetDefaultShapesColorsPreIntegrate();
				DebugDrawShapes(false, !!ChaosSolverDrawShapesShowKinematic, !!ChaosSolverDrawShapesShowDynamic);
			}
		}
		if (ChaosSolverDebugDrawPreIntegrationCollisions == 1)
		{
			DebugDraw::DrawCollisions(FRigidTransform3(), GetEvolution()->GetCollisionConstraints().GetConstraintAllocator(), 1.f, &ChaosSolverDebugDebugDrawSettings);
		}
#endif
	}

	void FPBDRigidsSolver::PreSolveDebugDraw(FReal Dt) const
	{
#if CHAOS_DEBUG_DRAW
		QUICK_SCOPE_CYCLE_COUNTER(SolverDebugDraw);

		if (ChaosSolverDebugDrawPostIntegrationShapes == 1)
		{
			if (bChaosDebugDraw_UseNewQueue)
			{
				for (FTransientGeometryParticleHandle& Particle : Particles.GetActiveKinematicParticlesView())
				{
					FChaosDDParticle::DrawShapes(Particle.Handle(), FColor::Blue);
				}
				for (FTransientGeometryParticleHandle& Particle : Particles.GetNonDisabledDynamicView())
				{
					FChaosDDParticle::DrawShapes(Particle.Handle(), FColor::Yellow);
				}
			}
			else
			{
				ChaosSolverDebugDebugDrawSettings.ShapesColorsPerState = DebugDraw::GetDefaultShapesColorsPostIntegrate();
				DebugDrawShapes(!!ChaosSolverDrawShapesShowStatic, !!ChaosSolverDrawShapesShowKinematic, !!ChaosSolverDrawShapesShowDynamic);
			}
		}
		if (ChaosSolverDebugDrawPostIntegrationCollisions == 1)
		{
			DebugDraw::DrawCollisions(FRigidTransform3(), GetEvolution()->GetCollisionConstraints().GetConstraintAllocator(), 1.f, &ChaosSolverDebugDebugDrawSettings);
		}
#endif
	}

	void FPBDRigidsSolver::PostTickDebugDraw(FReal Dt) const
	{
#if CHAOS_DEBUG_DRAW
		QUICK_SCOPE_CYCLE_COUNTER(SolverDebugDraw);

		// Don't debug draw resim frames
		if (GetEvolution()->IsResimming())
		{
			return;
		}

		if (CDDScene.IsValid())
		{
			// @todo(chaos): this logic should be in UChaosDebugDrawSubsystem but the CVars are here. Move them and this...
			const bool bRenderEnabled = (CDDScene->IsServer() && ChaosSolverDebugDrawShowServer) || (!CDDScene->IsServer() && ChaosSolverDebugDrawShowClient);
			CDDScene->SetRenderEnabled(bRenderEnabled);
		}

		const bool bIsServer = GetDebugName().ToString().StartsWith(TEXT("Server"));
		if (bIsServer && !ChaosSolverDebugDrawShowServer)
		{
			return;
		}
		if (!bIsServer && !ChaosSolverDebugDrawShowClient)
		{
			return;
		}

		if (ChaosSolverDebugDrawShapes == 1)
		{
			if (bChaosDebugDraw_UseNewQueue)
			{
				for (FTransientGeometryParticleHandle& Particle : Particles.GetNonDisabledView())
				{
					FChaosDDParticle::DrawShapes(Particle.Handle());
				}
			}
			else
			{
				if (ChaosSolverDebugDrawColorShapeByClientServer)
				{
					if (bIsServer)
					{
						ChaosSolverDebugDebugDrawSettings.ShapesColorsPerState = GetSolverShapesColorsByState_Server();
					}
					else
					{
						ChaosSolverDebugDebugDrawSettings.ShapesColorsPerState = GetSolverShapesColorsByState_Client();
					}
				}
				else
				{
					ChaosSolverDebugDebugDrawSettings.ShapesColorsPerState = DebugDraw::GetDefaultShapesColorsByState();
				}
				DebugDrawShapes(!!ChaosSolverDrawShapesShowStatic, !!ChaosSolverDrawShapesShowKinematic, !!ChaosSolverDrawShapesShowDynamic);
			}
		}
		if (ChaosSolverDebugDrawCollisions == 1)
		{
			DebugDraw::DrawCollisions(FRigidTransform3(), GetEvolution()->GetCollisionConstraints().GetConstraintAllocator(), 1.f, &ChaosSolverDebugDebugDrawSettings);
		}
		if (ChaosSolverDebugDrawMass == 1)
		{
			DebugDraw::DrawParticleMass(FRigidTransform3(), Particles.GetActiveKinematicParticlesView(), &ChaosSolverDebugDebugDrawSettings);
			DebugDraw::DrawParticleMass(FRigidTransform3(), Particles.GetNonDisabledDynamicView(), &ChaosSolverDebugDebugDrawSettings);
		}
		if (ChaosSolverDebugDrawDensity == 1)
		{
			DebugDraw::DrawParticleDensity(FRigidTransform3(), Particles.GetActiveKinematicParticlesView(), &ChaosSolverDebugDebugDrawSettings);
			DebugDraw::DrawParticleDensity(FRigidTransform3(), Particles.GetNonDisabledDynamicView(), &ChaosSolverDebugDebugDrawSettings);
		}
		if (ChaosSolverDebugDrawBVHs == 1)
		{
			for (const FTransientGeometryParticleHandle& Particle : Particles.GetNonDisabledView())
			{
				DebugDraw::DrawParticleBVH(FRigidTransform3(), Particle.Handle(), FColor::Silver, &ChaosSolverDebugDebugDrawSettings);
			}
		}
		if (ChaosSolverDebugDrawBounds == 1)
		{
			DebugDraw::DrawParticleBounds(FRigidTransform3(), Particles.GetActiveStaticParticlesView(), Dt, &ChaosSolverDebugDebugDrawSettings);
			DebugDraw::DrawParticleBounds(FRigidTransform3(), Particles.GetActiveKinematicParticlesView(), Dt, &ChaosSolverDebugDebugDrawSettings);
			DebugDraw::DrawParticleBounds(FRigidTransform3(), Particles.GetNonDisabledDynamicView(), Dt, &ChaosSolverDebugDebugDrawSettings);
		}
		if (ChaosSolverDrawTransforms == 1)
		{
			DebugDraw::DrawParticleTransforms(FRigidTransform3(), Particles.GetAllParticlesView(), &ChaosSolverDebugDebugDrawSettings);
		}
		if (ChaosSolverDrawIslands == 1)
		{
			DebugDraw::DrawConstraintGraph(FRigidTransform3(), GetEvolution()->GetIslandManager(), &ChaosSolverDebugDebugDrawSettings);
		}
		if (ChaosSolverDebugDrawIslandSleepState == 1)
		{
			GetEvolution()->GetIslandManager().DebugDrawSleepState(&ChaosSolverDebugDebugDrawSettings);
		}
		if (ChaosSolverDrawClusterConstraints == 1)
		{
			DebugDraw::DrawConnectionGraph(MEvolution->GetRigidClustering(), &ChaosSolverDebugDebugDrawSettings);
		}
		if (ChaosSolverDebugDrawCollidingShapes == 1)
		{
			DebugDraw::DrawCollidingShapes(FRigidTransform3(), GetEvolution()->GetCollisionConstraints(), 1.f, 0.f, &ChaosSolverDebugDebugDrawSettings);
		}
		if (ChaosSolverDrawJoints == 1)
		{
			DebugDraw::DrawJointConstraints(FRigidTransform3(), MEvolution->GetJointCombinedConstraints().LinearConstraints, 1.0f, ChaosSolverDrawJointFeatures, &ChaosSolverDebugDebugDrawSettings);
			DebugDraw::DrawJointConstraints(FRigidTransform3(), MEvolution->GetJointCombinedConstraints().NonLinearConstraints, 1.0f, ChaosSolverDrawJointFeatures, &ChaosSolverDebugDebugDrawSettings);
		}
		if (ChaosSolverDrawCharacterGroundConstraints == 1)
		{
			DebugDraw::DrawCharacterGroundConstraints(FRigidTransform3(), MEvolution->GetCharacterGroundConstraints(), &ChaosSolverDebugDebugDrawSettings);
		}
		if (ChaosSolverDebugDrawSpatialAccelerationStructure)
		{
			if (const auto SpatialAccelerationStructure = GetEvolution()->GetSpatialAcceleration())
			{
				DebugDraw::DrawSpatialAccelerationStructure(*SpatialAccelerationStructure, &ChaosSolverDebugDebugDrawSettings);
			}
		}
		if (ChaosSolverDebugDrawCollidingShapes == 1)
		{
			DebugDraw::DrawCollidingShapes(FRigidTransform3(), GetEvolution()->GetCollisionConstraints(), 1.f, 0.f, &ChaosSolverDebugDebugDrawSettings);
		}
		if (ChaosSolverDebugDrawSuspensionConstraints == 1)
		{
			DebugDraw::DrawSuspensionConstraints(FRigidTransform3(), GetEvolution()->GetSuspensionConstraints(), &ChaosSolverDebugDebugDrawSettings);
		}
#endif
	}

	FSingleParticlePhysicsProxy* FPBDRigidsSolver::GetParticleProxy_PT(const FUniqueIdx& Idx)
	{
		return SingleParticlePhysicsProxies_PT.IsValidIndex(Idx.Idx) ? SingleParticlePhysicsProxies_PT[Idx.Idx] : nullptr;
	}

	const FSingleParticlePhysicsProxy* FPBDRigidsSolver::GetParticleProxy_PT(const FUniqueIdx& Idx) const
	{
		return SingleParticlePhysicsProxies_PT.IsValidIndex(Idx.Idx) ? SingleParticlePhysicsProxies_PT[Idx.Idx] : nullptr;
	}

	FSingleParticlePhysicsProxy* FPBDRigidsSolver::GetParticleProxy_PT(const FGeometryParticleHandle& Handle)
	{
		return GetParticleProxy_PT(Handle.UniqueIdx());
	}

	const FSingleParticlePhysicsProxy* FPBDRigidsSolver::GetParticleProxy_PT(const FGeometryParticleHandle& Handle) const
	{
		return GetParticleProxy_PT(Handle.UniqueIdx());
	}

	void FPBDRigidsSolver::UpdateMaterial(FMaterialHandle InHandle, const FChaosPhysicsMaterial& InNewData)
	{
		TSolverSimMaterialScope<ELockType::Write> Scope(this);
		*SimMaterials.Get(InHandle.InnerHandle) = InNewData;
	}

	void FPBDRigidsSolver::CreateMaterial(FMaterialHandle InHandle, const FChaosPhysicsMaterial& InNewData)
	{
		TSolverSimMaterialScope<ELockType::Write> Scope(this);
		ensure(SimMaterials.Create(InNewData) == InHandle.InnerHandle);
	}

	void FPBDRigidsSolver::DestroyMaterial(FMaterialHandle InHandle)
	{
		TSolverSimMaterialScope<ELockType::Write> Scope(this);
		SimMaterials.Destroy(InHandle.InnerHandle);
	}

	void FPBDRigidsSolver::UpdateMaterialMask(FMaterialMaskHandle InHandle, const FChaosPhysicsMaterialMask& InNewData)
	{
		TSolverSimMaterialScope<ELockType::Write> Scope(this);
		*SimMaterialMasks.Get(InHandle.InnerHandle) = InNewData;
	}

	void FPBDRigidsSolver::CreateMaterialMask(FMaterialMaskHandle InHandle, const FChaosPhysicsMaterialMask& InNewData)
	{
		TSolverSimMaterialScope<ELockType::Write> Scope(this);
		ensure(SimMaterialMasks.Create(InNewData) == InHandle.InnerHandle);
	}

	void FPBDRigidsSolver::DestroyMaterialMask(FMaterialMaskHandle InHandle)
	{
		TSolverSimMaterialScope<ELockType::Write> Scope(this);
		SimMaterialMasks.Destroy(InHandle.InnerHandle);
	}

	void FPBDRigidsSolver::SyncQueryMaterials_External()
	{
		// Using lock on sim material is an imprefect workaround, we may block while physics thread is updating sim materials in callbacks.
		// QueryMaterials may be slightly stale. Need to rethink lifetime + ownership of materials for async case.
		//acquire external data lock
		FPhysicsSceneGuardScopedWrite ScopedWrite(GetExternalDataLock_External());
		TSolverSimMaterialScope<ELockType::Read> SimMatLock(this);
		
		QueryMaterials_External = SimMaterials;
		QueryMaterialMasks_External = SimMaterialMasks;
	}

	void FPBDRigidsSolver::UpdateExternalAccelerationStructure_External(ISpatialAccelerationCollection<FAccelerationStructureHandle,FReal,3>*& ExternalStructure)
	{
		GetEvolution()->UpdateExternalAccelerationStructure_External(ExternalStructure,*PendingSpatialOperations_External);
	}

	bool FPBDRigidsSolver::IsDetemerministic() const
	{
		return bIsDeterministic || (MRewindData != nullptr) || (ChaosSolverDeterministic >= 1) || FPhysicsSolverBase::IsNetworkPhysicsPredictionEnabled();
	}

	void FPBDRigidsSolver::SetIsDeterministic(const bool bInIsDeterministic)
	{
		if (bIsDeterministic != bInIsDeterministic)
		{
			bIsDeterministic = bInIsDeterministic;
			UpdateIsDeterministic();
		}
	}

	void FPBDRigidsSolver::UpdateIsDeterministic()
	{
		GetEvolution()->SetIsDeterministic(IsDetemerministic());
	}

	void FPBDRigidsSolver::SetParticleDynamicMisc(FPBDRigidParticleHandle* Rigid, const FParticleDynamicMisc& DynamicMisc)
	{
		if (Rigid == nullptr)
		{
			return;
		}

		// Enable or disable the particle
		if (Rigid->Disabled() != DynamicMisc.Disabled())
		{
			if (DynamicMisc.Disabled())
			{
				GetEvolution()->DisableParticle(Rigid);
			}
			else
			{
				GetEvolution()->EnableParticle(Rigid);
			}
		}

		// If we changed kinematics we need to rebuild the inertia conditioning
		const bool bDirtyInertiaConditioning = (Rigid->ObjectState() != DynamicMisc.ObjectState());
		if (bDirtyInertiaConditioning)
		{
			Rigid->SetInertiaConditioningDirty();
		}

		// A negative depenetration velocity means use the solver setting
		FRealSingle DepenetrationVelocity = DynamicMisc.InitialOverlapDepenetrationVelocity();
		if (DepenetrationVelocity < 0)
		{
			DepenetrationVelocity = GetEvolution()->GetCollisionConstraints().GetSolverSettings().DepenetrationVelocity;
		}

		Rigid->SetLinearEtherDrag(DynamicMisc.LinearEtherDrag());
		Rigid->SetAngularEtherDrag(DynamicMisc.AngularEtherDrag());
		Rigid->SetMaxLinearSpeedSq(DynamicMisc.MaxLinearSpeedSq());
		Rigid->SetMaxAngularSpeedSq(DynamicMisc.MaxAngularSpeedSq());
		Rigid->SetInitialOverlapDepenetrationVelocity(DepenetrationVelocity);
		Rigid->SetSleepThresholdMultiplier(DynamicMisc.SleepThresholdMultiplier());
		Rigid->SetCollisionGroup(DynamicMisc.CollisionGroup());
		Rigid->SetDisabled(DynamicMisc.Disabled());
		Rigid->SetCollisionConstraintFlags(DynamicMisc.CollisionConstraintFlags());
		Rigid->SetControlFlags(DynamicMisc.ControlFlags());
		Rigid->SetIterationSettings(DynamicMisc.IterationSettings());

		GetEvolution()->SetParticleObjectState(Rigid, DynamicMisc.ObjectState());
		GetEvolution()->SetParticleSleepType(Rigid, DynamicMisc.SleepType());

		GetEvolution()->GetIslandManager().FlagParticleUpdated(Rigid->Handle());
	}

	void FPBDRigidsSolver::ApplyConfig(const FChaosSolverConfiguration& InConfig)
	{
		GetEvolution()->GetRigidClustering().ApplySettings(InConfig);
		SetPositionIterations(InConfig.PositionIterations);
		SetVelocityIterations(InConfig.VelocityIterations);
		SetProjectionIterations(InConfig.ProjectionIterations);
		SetCollisionCullDistance(InConfig.CollisionCullDistance);
		SetCollisionMaxPushOutVelocity(InConfig.CollisionMaxPushOutVelocity);
		SetCollisionDepenetrationVelocity(InConfig.CollisionInitialOverlapDepenetrationVelocity);
		SetGenerateCollisionData(InConfig.bGenerateCollisionData);
		SetGenerateBreakingData(InConfig.bGenerateBreakData);
		SetGenerateTrailingData(InConfig.bGenerateTrailingData);
		SetCollisionFilterSettings(InConfig.CollisionFilterSettings);
		SetBreakingFilterSettings(InConfig.BreakingFilterSettings);
		SetTrailingFilterSettings(InConfig.TrailingFilterSettings);

		ApplyCVars();
	}

	void FPBDRigidsSolver::ApplyCVars()
	{
		MEvolution->GetCollisionConstraints().SetCollisionsEnabled(bChaosSolverCollisionEnabled);

		FCollisionDetectorSettings CollisionDetectorSettings = MEvolution->GetCollisionConstraints().GetDetectorSettings();
		CollisionDetectorSettings.bAllowManifoldReuse = (ChaosSolverCollisionAllowManifoldUpdate != 0);
		CollisionDetectorSettings.bDeferNarrowPhase = (ChaosSolverCollisionDeferNarrowPhase != 0);
		CollisionDetectorSettings.bAllowManifolds = (ChaosSolverCollisionUseManifolds != 0);
		CollisionDetectorSettings.bAllowCCD = bChaosUseCCD;
		CollisionDetectorSettings.bAllowMACD = bChaosUseMACD;
		MEvolution->GetCollisionConstraints().SetDetectorSettings(CollisionDetectorSettings);

		FPBDJointSolverSettings JointsSettings = MEvolution->GetJointCombinedConstraints().LinearConstraints.GetSettings();
		JointsSettings.MinSolverStiffness = ChaosSolverJointMinSolverStiffness;
		JointsSettings.MaxSolverStiffness = ChaosSolverJointMaxSolverStiffness;
		JointsSettings.NumIterationsAtMaxSolverStiffness = ChaosSolverJointNumIterationsAtMaxSolverStiffness;
		JointsSettings.PositionTolerance = ChaosSolverJointPositionTolerance;
		JointsSettings.AngleTolerance = ChaosSolverJointAngleTolerance;
		JointsSettings.MinParentMassRatio = ChaosSolverJointMinParentMassRatio;
		JointsSettings.MaxInertiaRatio = ChaosSolverJointMaxInertiaRatio;
		JointsSettings.bSolvePositionLast = bChaosSolverJointSolvePositionLast;
		JointsSettings.bUsePositionBasedDrives = bChaosSolverJointUsePositionBasedDrives;
		JointsSettings.NumShockPropagationIterations = ChaosSolverJointNumShockProagationIterations;
		JointsSettings.ShockPropagationOverride = ChaosSolverJointShockPropagation;
		JointsSettings.bSortEnabled = false;
		MEvolution->GetJointCombinedConstraints().LinearConstraints.SetSettings(JointsSettings);
		MEvolution->GetJointCombinedConstraints().NonLinearConstraints.SetSettings(JointsSettings);

		// Apply CVAR overrides if set
		{
			// To enable runtime support for switching collision features on/off we need to update existing constraints when config changes.
			if (bChaosCollisionConfigChanged)
			{
				// For now destroy the collisions. This is a bit over the top and causes problems for sleeping islands, but it's only for debugging/testing.
				GetEvolution()->DestroyTransientConstraints();
				bChaosCollisionConfigChanged = false;
			}

			if (ChaosSolverCollisionPositionFrictionIterations >= 0)
			{
				MEvolution->GetCollisionConstraints().SetPositionFrictionIterations(ChaosSolverCollisionPositionFrictionIterations);
			}
			if (ChaosSolverCollisionVelocityFrictionIterations >= 0)
			{
				MEvolution->GetCollisionConstraints().SetVelocityFrictionIterations(ChaosSolverCollisionVelocityFrictionIterations);
			}
			if (ChaosSolverSplitImpulseMode >= 0)
			{
				MEvolution->GetCollisionConstraints().SetUseSplitImpulse(ChaosSolverSplitImpulseMode > 0);
			}
			{
				MEvolution->SetShockPropagationIterations(ChaosSolverCollisionPositionShockPropagationIterations, ChaosSolverCollisionVelocityShockPropagationIterations);
			}
			if (ChaosSolverPositionIterations >= 0)
			{
				SetPositionIterations(ChaosSolverPositionIterations);
			}
			if (ChaosSolverVelocityIterations >= 0)
			{
				SetVelocityIterations(ChaosSolverVelocityIterations);
			}
			if (ChaosSolverProjectionIterations >= 0)
			{
				SetProjectionIterations(ChaosSolverProjectionIterations);
			}
			if (ChaosSolverCullDistance >= 0.0f)
			{
				SetCollisionCullDistance(ChaosSolverCullDistance);
			}
			if ((ChaosSolverVelocityBoundsMultiplier >= 0.0f) && (ChaosSolverMaxVelocityBoundsExpansion >= 0.0f))
			{
				SetVelocityBoundsExpansion(ChaosSolverVelocityBoundsMultiplier, ChaosSolverMaxVelocityBoundsExpansion);
			}
			if ((ChaosSolverVelocityBoundsMultiplierMACD >= 0.0f) && (ChaosSolverMaxVelocityBoundsExpansionMACD >= 0.0f))
			{
				SetVelocityBoundsExpansionMACD(ChaosSolverVelocityBoundsMultiplierMACD, ChaosSolverMaxVelocityBoundsExpansionMACD);
			}
			if (ChaosSolverMaxPushOutVelocity >= 0.0f)
			{
				SetCollisionMaxPushOutVelocity(ChaosSolverMaxPushOutVelocity);
			}
			if (ChaosSolverDepenetrationVelocity >= 0.0f)
			{
				SetCollisionDepenetrationVelocity(ChaosSolverDepenetrationVelocity);
			}
			if (ChaosSolverDeterministic >= 0)
			{
				UpdateIsDeterministic();
			}
		}
	}

	FPBDRigidsSolver::~FPBDRigidsSolver()
	{
		EventTeardown.Broadcast();
	}

	void FPBDRigidsSolver::FieldParameterUpdateCallback(
		FPBDPositionConstraints& PositionTarget,
		TMap<int32, int32>& TargetedParticles)
	{
		GetPerSolverField().FieldParameterUpdateCallback(this, PositionTarget, TargetedParticles);
	}

	void FPBDRigidsSolver::FieldForcesUpdateCallback()
	{
		GetPerSolverField().FieldForcesUpdateCallback(this);
	}

}; // namespace Chaos

