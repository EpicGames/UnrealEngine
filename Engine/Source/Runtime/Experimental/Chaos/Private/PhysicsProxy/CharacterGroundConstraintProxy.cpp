// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsProxy/CharacterGroundConstraintProxy.h"

#include "Chaos/Character/CharacterGroundConstraint.h"
#include "Chaos/Character/CharacterGroundConstraintContainer.h"
#include "Chaos/ParticleDirtyFlags.h"
#include "Chaos/PullPhysicsDataImp.h"
#include "PBDRigidsSolver.h"

//////////////////////////////////////////////////////////////////////////
/// Utility function

namespace
{
	Chaos::FGeometryParticleHandle* GetParticleHandleFromProxy(IPhysicsProxyBase* ProxyBase)
	{
		if (ProxyBase)
		{
			if (ProxyBase->GetType() == EPhysicsProxyType::SingleParticleProxy)
			{
				return ((Chaos::FSingleParticlePhysicsProxy*)ProxyBase)->GetHandle_LowLevel();
			}
		}
		return nullptr;
	}
}

//////////////////////////////////////////////////////////////////////////

namespace Chaos
{
	FCharacterGroundConstraintProxy::FCharacterGroundConstraintProxy(FCharacterGroundConstraint* InConstraintGT, FCharacterGroundConstraintHandle* InConstraintPT, UObject* InOwner)
		: Base(EPhysicsProxyType::CharacterGroundConstraintType, InOwner, MakeShared<FProxyTimestampBase>())
		, Constraint_GT(InConstraintGT)
		, Constraint_PT(InConstraintPT)
	{
		check(Constraint_GT != nullptr);
		Constraint_GT->SetProxy(this);
	}

	void FCharacterGroundConstraintProxy::InitializeOnPhysicsThread(FPBDRigidsSolver* InSolver, FDirtyPropertiesManager& Manager, int32 DataIdx, FDirtyChaosProperties& RemoteData)
	{
		TGeometryParticleHandles<FReal,3>& Handles = InSolver->GetParticles().GetParticleHandles();
		if (Handles.Size() > 0)
		{
			check(InSolver);
			FCharacterGroundConstraintContainer& ConstraintContainer = InSolver->GetCharacterGroundConstraints();

			FGeometryParticleHandle* CharacterHandle = nullptr;
			if (const FParticleProxyProperty* CharacterProxy = RemoteData.FindCharacterParticleProxy(Manager, DataIdx))
			{
				CharacterHandle = GetParticleHandleFromProxy(CharacterProxy->ParticleProxy);
			}

			FGeometryParticleHandle* GroundHandle = nullptr;
			if (const FParticleProxyProperty* GroundProxy = RemoteData.FindGroundParticleProxy(Manager, DataIdx))
			{
				GroundHandle = GetParticleHandleFromProxy(GroundProxy->ParticleProxy);
			}

			// Constraint only requires that the character particle be set
			if (CharacterHandle)
			{
				const FCharacterGroundConstraintSettings* ConstraintSettings = RemoteData.FindCharacterGroundConstraintSettings(Manager, DataIdx);
				const FCharacterGroundConstraintDynamicData* ConstraintData = RemoteData.FindCharacterGroundConstraintDynamicData(Manager, DataIdx);

				if (ConstraintSettings && ConstraintData)
				{
					Constraint_PT = ConstraintContainer.AddConstraint(*ConstraintSettings, *ConstraintData, CharacterHandle, GroundHandle);
				}
			}
		}
	}

	void FCharacterGroundConstraintProxy::PushStateOnPhysicsThread(FPBDRigidsSolver* InSolver, const FDirtyPropertiesManager& Manager, int32 DataIdx, const FDirtyChaosProperties& RemoteData)
	{
		if (Constraint_PT && Constraint_PT->IsValid())
		{
			Constraint_PT->ClearDirtyFlags();

			if (const FCharacterGroundConstraintSettings* Settings = RemoteData.FindCharacterGroundConstraintSettings(Manager, DataIdx))
			{
				Constraint_PT->Settings = *Settings;
			}

			if (const FCharacterGroundConstraintDynamicData* Data = RemoteData.FindCharacterGroundConstraintDynamicData(Manager, DataIdx))
			{
				Constraint_PT->Data = *Data;
			}

			if (const FParticleProxyProperty* GroundParticleProxy = RemoteData.FindGroundParticleProxy(Manager, DataIdx))
			{
				if (FGeometryParticleHandle* Handle = GetParticleHandleFromProxy(GroundParticleProxy->ParticleProxy))
				{
					Constraint_PT->GroundParticle = Handle;
					Constraint_PT->SetDirtyFlag(FCharacterGroundConstraintHandle::EDirtyDataFlags::GroundParticle);
				}
			}
		}
	}

	void FCharacterGroundConstraintProxy::PushStateOnGameThread(FDirtyPropertiesManager& Manager, int32 DataIdx, FDirtyChaosProperties& RemoteData)
	{
		if (Constraint_GT && Constraint_GT->IsValid())
		{
			Constraint_GT->SyncRemoteData(Manager, DataIdx, RemoteData);
		}
	}

	void FCharacterGroundConstraintProxy::DestroyOnGameThread()
	{
		if (Constraint_GT)
		{
			delete Constraint_GT;
			Constraint_GT = nullptr;
		}
	}

	void FCharacterGroundConstraintProxy::DestroyOnPhysicsThread(FPBDRigidsSolver* InSolver)
	{
		if (Constraint_PT)
		{
			if (FGeometryParticleHandle* CharacterParticle = Constraint_PT->GetCharacterParticle())
			{
				CharacterParticle->RemoveConstraintHandle(Constraint_PT);
			}
			if (FGeometryParticleHandle* GroundParticle = Constraint_PT->GetGroundParticle())
			{
				GroundParticle->RemoveConstraintHandle(Constraint_PT);
			}

			InSolver->GetEvolution()->RemoveConstraintFromConstraintGraph(Constraint_PT);
			FCharacterGroundConstraintContainer& Constraints = InSolver->GetCharacterGroundConstraints();
			Constraints.RemoveConstraint(Constraint_PT);
			Constraint_PT = nullptr;
		}
	}

	void FCharacterGroundConstraintProxy::BufferPhysicsResults(FDirtyCharacterGroundConstraintData& Buffer)
	{
		Buffer.SetProxy(*this);
		if (Constraint_PT != nullptr && Constraint_PT->IsValid() && Constraint_PT->IsEnabled())
		{
			Buffer.Force = Constraint_PT->GetSolverAppliedForce();
			Buffer.Torque = Constraint_PT->GetSolverAppliedTorque();

			Buffer.bSettingsChanged = Constraint_PT->HaveSettingsChanged();
			if (Buffer.bSettingsChanged)
			{
				Buffer.Settings = Constraint_PT->GetSettings();
			}

			Buffer.bDataChanged = Constraint_PT->HasDataChanged();
			if (Buffer.bDataChanged)
			{
				Buffer.Data = Constraint_PT->GetData();
			}

			Buffer.GroundParticle = nullptr;
			Buffer.bGroundParticleChanged = Constraint_PT->HasGroundParticleChanged();
			if (Buffer.bGroundParticleChanged)
			{
				if (FGeometryParticleHandle* GroundParticle = Constraint_PT->GetGroundParticle())
				{
					if (FPBDRigidParticleHandle* Rigid = GroundParticle->CastToRigidParticle())
					{
						if (!Rigid->Disabled())
						{
							Buffer.GroundParticle = GroundParticle;
						}
					}
				}
			}

			Constraint_PT->ClearDirtyFlags();
		}
	}

	bool FCharacterGroundConstraintProxy::PullFromPhysicsState(const FDirtyCharacterGroundConstraintData& Buffer, const int32 SolverSyncTimestamp)
	{
		if (Constraint_GT != nullptr && Constraint_GT->IsValid())
		{
			Constraint_GT->SolverAppliedForce = Buffer.Force;
			Constraint_GT->SolverAppliedTorque = Buffer.Torque;

			if (Buffer.bDataChanged)
			{
				Constraint_GT->ConstraintData.Modify(false, Constraint_GT->DirtyFlags, Constraint_GT->Proxy, [&Buffer](FCharacterGroundConstraintDynamicData& Data) {
					Data = Buffer.Data;
					});
			}

			if (Buffer.bSettingsChanged)
			{
				Constraint_GT->ConstraintSettings.Modify(false, Constraint_GT->DirtyFlags, Constraint_GT->Proxy, [&Buffer](FCharacterGroundConstraintSettings& Settings) {
					Settings = Buffer.Settings;
					});
			}

			// Temporarily disabling the write back of the ground particle until we can fix it crashing with an invalid proxy

			//if (Buffer.bGroundParticleChanged)
			//{
			//	if (Buffer.GroundParticle && (Buffer.GroundParticle->ParticleIdx != INDEX_NONE))
			//	{
			//		FSingleParticlePhysicsProxy* NewGroundProxy = (FSingleParticlePhysicsProxy*)(Buffer.GroundParticle->PhysicsProxy());
			//		Constraint_GT->GroundProxy.Modify(false, Constraint_GT->DirtyFlags, Constraint_GT->Proxy, [NewGroundProxy](FParticleProxyProperty& Data)
			//			{
			//				Data.ParticleProxy = NewGroundProxy;
			//			});
			//	}
			//	else
			//	{
			//		Constraint_GT->GroundProxy.Modify(false, Constraint_GT->DirtyFlags, Constraint_GT->Proxy, [](FParticleProxyProperty& Data)
			//			{
			//				Data.ParticleProxy = nullptr;
			//			});
			//	}
			//}
		}

		return true;
	}

} // namespace Chaos