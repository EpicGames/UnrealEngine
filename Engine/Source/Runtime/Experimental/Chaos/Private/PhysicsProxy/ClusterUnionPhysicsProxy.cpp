// Copyright Epic Games, Inc. All Rights Reserved.
#include "PhysicsProxy/ClusterUnionPhysicsProxy.h"

#include "Chaos/ClusterUnionManager.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/PhysicsObjectInternal.h"
#include "Chaos/PhysicsObjectInternalInterface.h"
#include "Chaos/PullPhysicsDataImp.h"
#include "Math/UnrealMathUtility.h"
#include "PBDRigidsSolver.h"
#include "Chaos/DebugDrawQueue.h"

namespace Chaos
{
	static int32 GShallowCopyClusterUnionGeometryOnUpdate = 1;
	FAutoConsoleVariableRef CVar_ShallowCopyClusterUnionGeometryOnUpdate(TEXT("p.ShallowCopyOnClusterUnionUpdate"), GShallowCopyClusterUnionGeometryOnUpdate, TEXT("If 1, shallow copy the root union geometry of a cluster union when its geometry updates, otherwise deep copy the geometry hierarchy"));

	static int32 GFixLockedChildXROnClusterUnionPush = 0;
	FAutoConsoleVariableRef CVar_FixLockedChildXROnClusterUnionPush(
		TEXT("p.Chaos.ClusterUnion.FixLockedChildXROnPush"),
		GFixLockedChildXROnClusterUnionPush,
		TEXT("If 1, after a cluster union receives a parent XR push, re-align locked children's cached X/R to ChildToParent * NewParentTM. Fixes a wrong-CoM bug when a child Remove lands on the same tick as a parent XR replication."));

	namespace CVars
	{
		extern bool bChaosEnableOverrideSingleParticleTypeToCluster;
	}

	namespace
	{
		FPBDRigidsEvolutionGBF* GetEvolution(FClusterUnionPhysicsProxy* Proxy)
		{
			if (!Proxy)
			{
				return nullptr;
			}

			FPBDRigidsSolver* RigidsSolver = Proxy->GetSolver<FPBDRigidsSolver>();
			if (!RigidsSolver)
			{
				return nullptr;
			}

			return RigidsSolver->GetEvolution();
		}

		template<typename TParticle>
		void BufferPhysicsResultsImp(FClusterUnionPhysicsProxy* Proxy, TParticle* Particle, FDirtyClusterUnionData& BufferData)
		{
			if (!Particle || !Proxy)
			{
				return;
			}

			BufferData.SetProxy(*Proxy);
			BufferData.X = Particle->GetX();
			BufferData.R = Particle->GetR();
			BufferData.V = Particle->GetV();
			BufferData.W = Particle->GetW();
			BufferData.Mass = FRealSingle(Particle->M());
			BufferData.Inertia = FVec3f(Particle->I());
			BufferData.ObjectState = Particle->ObjectState();
			BufferData.Geometry = nullptr;

			const FShapesArray& ShapeArray = Particle->ShapesArray();
			BufferData.CollisionData.Empty(ShapeArray.Num());
			BufferData.ShapeFilterData.Empty(ShapeArray.Num());
			BufferData.ShapeInstanceData.Empty(ShapeArray.Num());

			for (const TUniquePtr<Chaos::FPerShapeData>& ShapeData : ShapeArray)
			{
				BufferData.CollisionData.Add(ShapeData->GetCollisionData());
				BufferData.ShapeFilterData.Add(ShapeData->GetShapeFilterData());
				BufferData.ShapeInstanceData.Add(ShapeData->GetFilterInstanceData());
			}

			if constexpr (std::is_base_of_v<FClusterUnionPhysicsProxy::FInternalParticle, TParticle>)
			{
				BufferData.bIsAnchored = Particle->IsAnchored();
			}
		}
	}

	FClusterUnionPhysicsProxy::FClusterUnionPhysicsProxy(UObject* InOwner, const FClusterCreationParameters& InParameters, const FClusterUnionInitData& InInitData)
		: Base(InOwner)
		, ClusterParameters(InParameters)
		, InitData(InInitData)
	{
		InterpolationData = MakeUnique<FProxyInterpolationBase>();
	}

	void FClusterUnionPhysicsProxy::Initialize_External()
	{
		// Create game thread particle as well as the physics object.
		Particle_External = FExternalParticle::CreateParticle();
		check(Particle_External != nullptr);

#if CHAOS_DEBUG_NAME
		Particle_External->SetDebugName(InitData.DebugName);
#endif

		Particle_External->SetProxy(this);
		Particle_External->SetUserData(InitData.UserData);
		Particle_External->SetX(InitData.InitialTransform.GetTranslation());
		Particle_External->SetR(InitData.InitialTransform.GetRotation());
		Particle_External->SetCCDEnabled(InitData.bCCDEnabled);
		Particle_External->SetMACDEnabled(InitData.bMACDEnabled);

		// NO DIRTY FLAGS ALLOWED. We must strictly manage the dirty flags on the particle.
		// Setting the particle's XR on the particle will set the XR dirty flag but that isn't
		// used for the cluster union (there is no functionality in Chaos to let the particle
		// be easily managed by a proxy that isn't the single particle physics proxy). And if the
		// XR dirty flag is set, we'll try to access buffers that don't exist for cluster union proxies.
		Particle_External->ClearDirtyFlags();
		PhysicsObject = FPhysicsObjectFactory::CreatePhysicsObject(this);
	}

	void FClusterUnionPhysicsProxy::Initialize_Internal(FPBDRigidsSolver* RigidsSolver, FPBDRigidsSolver::FParticlesType& Particles)
	{
		if (!ensure(RigidsSolver) || !ensure(Particle_External != nullptr))
		{
			return;
		}

		bIsInitializedOnPhysicsThread = true;

		bEnableStrainOnCollision_Internal = ClusterParameters.bEnableStrainOnCollision;

		FPBDRigidsEvolutionGBF* Evolution = RigidsSolver->GetEvolution();
		if (!ensure(Evolution))
		{
			return;
		}

		FUniqueIdx UniqueIndex = Particle_External->UniqueIdx();
		FClusterUnionManager& ClusterUnionManager = Evolution->GetRigidClustering().GetClusterUnionManager();

		FClusterUnionCreationParameters ClusterUnionParameters;
		ClusterUnionParameters.UniqueIndex = &UniqueIndex;
		ClusterUnionParameters.ActorId = InitData.ActorId;
		ClusterUnionParameters.ComponentId = InitData.ComponentId;
		ClusterUnionParameters.GravityGroupOverride = InitData.GravityGroupOverride;

		ClusterUnionIndex = ClusterUnionManager.CreateNewClusterUnion(ClusterParameters, ClusterUnionParameters);
		if (FClusterUnion* ClusterUnion = ClusterUnionManager.FindClusterUnion(ClusterUnionIndex); ensure(ClusterUnion != nullptr))
		{
			Particle_Internal = ClusterUnion->InternalCluster;
			Particle_Internal->SetPhysicsProxy(this);
			Particle_Internal->GTGeometryParticle() = Particle_External.Get();
			Particle_Internal->SetUnbreakable(InitData.bUnbreakable);
			Particle_Internal->SetX(InitData.InitialTransform.GetTranslation());
			Particle_Internal->SetR(InitData.InitialTransform.GetRotation());
			Particle_Internal->SetVf(Chaos::FVec3f(0.f));
			Particle_Internal->SetWf(Chaos::FVec3f(0.f));
			Particle_Internal->SetP(Particle_Internal->GetX());
			Particle_Internal->SetQf(Particle_Internal->GetRf());
			Particle_Internal->SetCenterOfMass(FVector3f::ZeroVector);
			Particle_Internal->SetRotationOfMass(FQuat::Identity);
			Particle_Internal->SetCCDEnabled(InitData.bCCDEnabled);
			Particle_Internal->SetMACDEnabled(InitData.bMACDEnabled);

#if CHAOS_DEBUG_NAME
			if (Particle_External.IsValid())
			{
				Particle_Internal->SetDebugName(Particle_External->DebugName());
			}
#endif

			// For explicit cluster unions (i.e. cluster unions created via a cluster union component rather than the cluster group index), the cluster union itself is responsible for initializing
			// the particle's XR from the component's transform. Therefore, we never want the PT to try and initialize the XR of the cluster union - the only changes to XR from the PT should
			// be a result of physical simulation.
			ClusterUnion->bNeedsXRInitialization = false;
			ClusterUnion->bCheckConnectivity = InitData.bCheckConnectivity;
			ClusterUnion->bGenerateConnectivityEdges = InitData.bGenerateConnectivityEdges;
		}
	}

	bool FClusterUnionPhysicsProxy::HasChildren_Internal() const
	{
		if (!Particle_Internal)
		{
			return false;
		}

		return Particle_Internal->ClusterIds().NumChildren > 0;
	}

	void FClusterUnionPhysicsProxy::AddPhysicsObjects_External(const TArray<FPhysicsObjectHandle>& Objects)
	{
		if (!Solver)
		{
			return;
		}
		
		Solver->EnqueueCommandImmediate(
			[this, Objects=Objects]() mutable
			{
				FReadPhysicsObjectInterface_Internal Interface = FPhysicsObjectInternalInterface::GetRead();
				if (FPBDRigidsEvolutionGBF* Evolution = GetEvolution(this))
				{
					Evolution->GetRigidClustering().GetClusterUnionManager().AddPendingClusterIndexOperation(ClusterUnionIndex, EClusterUnionOperation::Add, Interface.GetAllRigidParticles(Objects));
				}
			}
		);
	}

	void FClusterUnionPhysicsProxy::RemovePhysicsObjects_External(const TSet<FPhysicsObjectHandle>& Objects)
	{
		if (!Solver || Objects.IsEmpty())
		{
			return;
		}

		Solver->EnqueueCommandImmediate(
			[this, Objects = Objects]() mutable
			{
				FReadPhysicsObjectInterface_Internal Interface = FPhysicsObjectInternalInterface::GetRead();
				if (FPBDRigidsEvolutionGBF* Evolution = GetEvolution(this))
				{
					Evolution->GetRigidClustering().GetClusterUnionManager().AddPendingClusterIndexOperation(ClusterUnionIndex, EClusterUnionOperation::Remove, Interface.GetAllRigidParticles(Objects.Array()));
				}
			}
		);
	}

	void FClusterUnionPhysicsProxy::SetIsAnchored_External(bool bIsAnchored)
	{
		if (!Solver || !ensure(Particle_External))
		{
			return;
		}

		Solver->EnqueueCommandImmediate(
			[this, bIsAnchored]() mutable
			{
				if (Particle_Internal)
				{
					if (FPBDRigidsEvolutionGBF* Evolution = GetEvolution(this))
					{
						const bool bWasAnchored = Particle_Internal->IsAnchored();
						Particle_Internal->SetIsAnchored(bIsAnchored);

						// We also need to make sure the cluster union won't try to override this value that we set.
						Chaos::FClusterUnionManager& Manager = Evolution->GetRigidClustering().GetClusterUnionManager();
						if (Chaos::FClusterUnion* ClusterUnion = Manager.FindClusterUnion(ClusterUnionIndex))
						{
							ClusterUnion->bAnchorLock = true;
						}

						// Let the cluster union manager apply the proper properties on the particle.
						if (bWasAnchored != bIsAnchored)
						{
							Manager.RequestDeferredClusterPropertiesUpdate(ClusterUnionIndex, Chaos::EUpdateClusterUnionPropertiesFlags::UpdateKinematicProperties);
						}
					}
				}
			}
		);
	}

	EObjectStateType FClusterUnionPhysicsProxy::GetObjectState_External() const
	{
		if (!ensure(Particle_External))
		{
			return EObjectStateType::Uninitialized;
		}

		return Particle_External->ObjectState();
	}

	void FClusterUnionPhysicsProxy::SetObjectState_External(EObjectStateType State)
	{
		if (!Solver || !ensure(Particle_External))
		{
			return;
		}

		Solver->EnqueueCommandImmediate(
			[this, State]() mutable
			{
				if (Particle_Internal)
				{
					if (Particle_Internal->ObjectState() != State)
					{
						if (FPBDRigidsEvolutionGBF* Evolution = GetEvolution(this))
						{
							Evolution->SetParticleObjectState(Particle_Internal, State);
						}
					}
				}
			}
		);
	}

	ESleepType FClusterUnionPhysicsProxy::GetSleepType_External() const
	{
		if (!ensure(Particle_External))
		{
			return ESleepType::MaterialSleep;
		}

		return Particle_External->SleepType();
	}

	void FClusterUnionPhysicsProxy::SetSleepType_External(const ESleepType SleepType)
	{
		if (!Solver || !ensure(Particle_External))
		{
			return;
		}

		Solver->EnqueueCommandImmediate(
			[this, SleepType]() mutable
			{
				if (Particle_Internal)
				{
					if (Particle_Internal->SleepType() != SleepType)
					{
						if (FPBDRigidsEvolutionGBF* Evolution = GetEvolution(this))
						{
							Evolution->SetParticleSleepType(Particle_Internal, SleepType);
						}
					}
				}
			}
		);
	}

	bool FClusterUnionPhysicsProxy::GetDisabled_External() const
	{
		if (!ensure(Particle_External))
		{
			return false;
		}
				
		return Particle_External->Disabled();
	}

	void FClusterUnionPhysicsProxy::SetDisabled_External(const bool Disabled)
	{
		if (!Solver || !ensure(Particle_External))
		{
			return;
		}

		Solver->EnqueueCommandImmediate(
			[this, Disabled]() mutable
			{
				if (Particle_Internal)
				{
					if (Particle_Internal->Disabled() != Disabled)
					{
						if (FPBDRigidsEvolutionGBF* Evolution = GetEvolution(this))
						{
							if (Disabled)
							{
								Evolution->DisableParticle(Particle_Internal);
							}
							else
							{
								Evolution->EnableParticle(Particle_Internal);
							}
						}
					}
				}
			}
		);
	}

	bool FClusterUnionPhysicsProxy::GetGravityEnabled_External() const
	{
		if (!ensure(Particle_External))
		{
			return false;
		}

		return Particle_External->GravityEnabled();
	}

	void FClusterUnionPhysicsProxy::SetGravityEnabled_External(const bool Enabled)
	{
		if (!Solver || !ensure(Particle_External))
		{
			return;
		}

		Solver->EnqueueCommandImmediate(
			[this, Enabled]() mutable
			{
				if (Particle_Internal)
				{
					Particle_Internal->SetGravityEnabled(Enabled);
				}
			}
		);
	}
	
	void FClusterUnionPhysicsProxy::Wake_External()
	{
		if (!Solver || !ensure(Particle_External))
		{
			return;
		}

		Solver->EnqueueCommandImmediate(
			[this]() mutable
			{
				if (Particle_Internal)
				{
					if (FPBDRigidsEvolutionGBF* Evolution = GetEvolution(this))
					{
						Evolution->WakeParticle(Particle_Internal);
					}
				}
			}
		);
	}

	void FClusterUnionPhysicsProxy::SetMass_External(Chaos::FReal InMass)
	{
		if (!Solver || !ensure(Particle_External))
		{
			return;
		}

		Chaos::FRealSingle Mass = Chaos::FRealSingle(InMass);
		Chaos::FRealSingle InvMass = 0;
		Chaos::FVec3f Inertia = FVec3f(0);
		Chaos::FVec3f InvInertia = FVec3f(0);
		if (Mass > 0)
		{
			InvMass = 1.0f / Mass;

			Chaos::FRealSingle OldMass = Chaos::FRealSingle(Particle_External->M());
			if (OldMass > 0)
			{
				const Chaos::FRealSingle MassScale = Mass / OldMass;
				Inertia = Particle_External->I() * MassScale;
				InvInertia = Particle_External->InvI() / MassScale;
			}
		}

		if (Particle_External)
		{
			Particle_External->SetM(Mass);
			Particle_External->SetInvM(InvMass);
			Particle_External->SetI(Inertia);
			Particle_External->SetInvI(InvInertia);
		}

		Solver->EnqueueCommandImmediate(
			[this, Mass, InvMass, Inertia, InvInertia]()
			{
				if (Particle_Internal)
				{
					Particle_Internal->SetM(Mass);
					Particle_Internal->SetInvM(InvMass);
					Particle_Internal->SetI(Inertia);
					Particle_Internal->SetInvI(InvInertia);
				}
			}
		);
	}

	void FClusterUnionPhysicsProxy::RemoveShapes_External(const TArray<FPBDRigidParticle*>& ShapeParticles)
    {
		RemoveParticlesFromClusterUnionGeometry(Particle_External.Get(), ShapeParticles, GeometryChildParticles_External);
    }
	
	void FClusterUnionPhysicsProxy::MergeGeometry_External(TArray<Chaos::FImplicitObjectPtr>&& ImplicitGeometries, const TArray<FPBDRigidParticle*>& ShapeParticles)
	{
		if (!ensure(Particle_External) || !ensure(ImplicitGeometries.Num() == ShapeParticles.Num()))
		{
			return;
		}
		
		// In the cases where this is necessary, the SQ should have a valid state - it's just the geometry itself that isn't valid.
		ModifyAdditionOfChildrenToClusterUnionGeometry(
			Particle_External.Get(),
			ShapeParticles,
			InitData.ActorId,
			InitData.ComponentId,
			[this, ImplicitGeometries=MoveTemp(ImplicitGeometries)]() mutable
			{
				Particle_External->MergeGeometry(MoveTemp(ImplicitGeometries));
			}
		);

		GeometryChildParticles_External.Append(ShapeParticles);
	}

	void FClusterUnionPhysicsProxy::SetGeometry_External(const Chaos::FImplicitObjectPtr& Geometry, const TArray<FPBDRigidParticle*>& ShapeParticles)
	{
		if (!ensure(Particle_External))
		{
			return;
		}

		// In the cases where this is necessary, the SQ should have a valid state - it's just the geometry itself that isn't valid.
		ModifyAdditionOfChildrenToClusterUnionGeometry(
			Particle_External.Get(),
			ShapeParticles,
			InitData.ActorId,
			InitData.ComponentId,
			[this, &Geometry]()
			{
				Particle_External->SetGeometry(Geometry);
			}
		);

		GeometryChildParticles_External = ShapeParticles;
	}

	DECLARE_CYCLE_STAT(TEXT("FClusterUnionPhysicsProxy::PushToPhysicsState"), STAT_ClusterUnionPhysicsProxyPushToPhysicsState, STATGROUP_Chaos);
	void FClusterUnionPhysicsProxy::PushToPhysicsState(const FDirtyPropertiesManager& Manager, int32 DataIdx, const FDirtyProxy& Dirty)
	{
		SCOPE_CYCLE_COUNTER(STAT_ClusterUnionPhysicsProxyPushToPhysicsState);
		if (!ensure(Solver) || !ensure(Particle_Internal))
		{
			return;
		}

		const FDirtyChaosProperties& ParticleData = Dirty.PropertyData;
		FPBDRigidsSolver& RigidsSolver = *static_cast<FPBDRigidsSolver*>(Solver);
		if (!ensure(RigidsSolver.GetEvolution()))
		{
			return;
		}

		FPBDRigidsEvolutionGBF& Evolution = *RigidsSolver.GetEvolution();

		if (const FParticlePositionRotation* NewXR = ParticleData.FindClusterXR(Manager, DataIdx))
		{
			// If we change the cluster union's location and the child particles are not soft locked with a given child to parent,
			// our goal is to maintain the particle's location rather than its child to parent.
			FTransform NewTransform{ NewXR->R(), NewXR->X() };

			TArray<FPBDRigidParticleHandle*> ParticlesToUpdate;
			TArray<FTransform> NewChildToParent;

			FClusterUnionManager& ClusterUnionManager = Evolution.GetRigidClustering().GetClusterUnionManager();
			if (FClusterUnion* Union = ClusterUnionManager.FindClusterUnion(ClusterUnionIndex); Union && !Union->bNeedsXRInitialization)
			{
				for (FPBDRigidParticleHandle* Particle : Union->ChildParticles)
				{
					if (FPBDRigidClusteredParticleHandle* ClusterParticle = Particle->CastToClustered())
					{
						if (!ClusterParticle->IsChildToParentLocked())
						{
							ParticlesToUpdate.Add(Particle);

							const FRigidTransform3 ChildWorldTM(Particle->GetX(), Particle->GetR());
							NewChildToParent.Add(ChildWorldTM.GetRelativeTransform(NewTransform));
						}
					}
				}
			}

			Evolution.SetParticleTransform(Particle_Internal, NewXR->GetX(), NewXR->R(), true);

			// Re-align LOCKED children's cached X/R to the new parent transform.
			// Their ChildToParent is authoritative; their cached world X/R must follow the
			// parent so that UpdateClusterMassProperties (which derives ChildToParent from
			// current X/R) doesn't read a stale value relative to the just-updated parent
			// and bake a wrong CoM offset on a same-tick Remove.
			if (GFixLockedChildXROnClusterUnionPush)
			{
				if (FClusterUnion* LockedUnion = ClusterUnionManager.FindClusterUnion(ClusterUnionIndex); LockedUnion && !LockedUnion->bNeedsXRInitialization)
				{
					for (FPBDRigidParticleHandle* Particle : LockedUnion->ChildParticles)
					{
						FPBDRigidClusteredParticleHandle* ClusterParticle = Particle ? Particle->CastToClustered() : nullptr;
						if (!ClusterParticle || !ClusterParticle->IsChildToParentLocked())
						{
							continue;
						}

						const FRigidTransform3 NewChildWorldTM = ClusterParticle->ChildToParent() * NewTransform;
						ClusterParticle->SetX(NewChildWorldTM.GetTranslation());
						ClusterParticle->SetP(NewChildWorldTM.GetTranslation());
						ClusterParticle->SetR(NewChildWorldTM.GetRotation());
						ClusterParticle->SetQ(NewChildWorldTM.GetRotation());

						Evolution.GetParticles().MarkTransientDirtyParticle(ClusterParticle);
					}
				}
			}

			if (!ParticlesToUpdate.IsEmpty() && !NewChildToParent.IsEmpty())
			{
				// Do not lock this child to parent as it's not coming from the server. It's just to smooth out the XR replication on the client
				// while we still have not yet received the ChildToParent yet.
				ClusterUnionManager.UpdateClusterUnionParticlesChildToParent(ClusterUnionIndex, ParticlesToUpdate, NewChildToParent, false);
			}

			// Particle needs to actually be marked dirty! Less so to pull state from the PT back to the GT for the cluster union
			// but this is primarily for proxies within the cluster union. They (i.e. GCs) need to see this change to the particle's transform.
			Evolution.GetParticles().MarkTransientDirtyParticle(Particle_Internal);

			const FRigidTransform3 WorldTransform{ Particle_Internal->GetX(), Particle_Internal->GetR() };
			Particle_Internal->UpdateWorldSpaceState(WorldTransform, FVec3(0));

			Evolution.DirtyParticle(*Particle_Internal);
		}

		if (const FParticleVelocities* NewVelocities = ParticleData.FindClusterVelocities(Manager, DataIdx))
		{
			Evolution.SetParticleVelocities(Particle_Internal, NewVelocities->V(), NewVelocities->W());
		}
	}

	DECLARE_CYCLE_STAT(TEXT("FClusterUnionPhysicsProxy::PullFromPhysicsState"), STAT_ClusterUnionPhysicsProxyPullFromPhysicsState, STATGROUP_Chaos);
	bool FClusterUnionPhysicsProxy::PullFromPhysicsState(const FDirtyClusterUnionData& PullData, int32 SolverSyncTimestamp, const FDirtyClusterUnionData* NextPullData, const FRealSingle* Alpha, const FDirtyRigidParticleReplicationErrorData* Error, const Chaos::FReal AsyncFixedTimeStep)
	{
		SCOPE_CYCLE_COUNTER(STAT_ClusterUnionPhysicsProxyPullFromPhysicsState);
		if (!ensure(Particle_External))
		{
			return false;
		}

		const FDirtyClusterUnionData& CurrentPullData = NextPullData ? *NextPullData : PullData;
		const FClusterUnionProxyTimestamp* ProxyTimestamp = CurrentPullData.GetTimestamp();
		if (!ProxyTimestamp)
		{
			return false;
		}

		FProxyInterpolationBase* InterpData = GetInterpolationData();
		if (Error)
		{
			InterpData = GetOrCreateErrorInterpolationData<FProxyInterpolationError>();
			check(InterpData);

			const FReal ErrorDistanceSqr = Error->ErrorX.SizeSquared();
			FReal ErrorCorrectionDuration = RenderInterpolationCVars::RenderInterpErrorCorrectionDuration;
			FReal MaxErrorCorrection = RenderInterpolationCVars::RenderInterpMaximumErrorCorrectionBeforeSnapping;
			FReal MaxErrorDesyncTime = RenderInterpolationCVars::RenderInterpMaximumErrorCorrectionDesyncTimeBeforeSnapping;
			if (FErrorInterpolationSettings* ErrorSettings = InterpData->GetErrorInterpolationSettings())
			{
				ErrorCorrectionDuration = ErrorSettings->ErrorCorrectionDuration;
				MaxErrorCorrection = ErrorSettings->MaximumErrorCorrectionBeforeSnapping;
				MaxErrorDesyncTime = ErrorSettings->MaximumErrorCorrectionDesyncTimeBeforeSnapping;
			}
			MaxErrorCorrection = MaxErrorCorrection * MaxErrorCorrection; // Square the value

			auto VelocitySnapLimitHelper = [ErrorDistanceSqr, MaxErrorDesyncTime](const FVec3& Velocity) -> const bool
				{
					if (MaxErrorDesyncTime <= 0.0)
					{
						return false;
					}
					return ErrorDistanceSqr < (Velocity * MaxErrorDesyncTime).SizeSquared();
				};

			// Accumulate error if within interpolation limits
			if (ErrorDistanceSqr < MaxErrorCorrection || VelocitySnapLimitHelper(Particle_External->V()) || VelocitySnapLimitHelper(PullData.V) || (NextPullData && VelocitySnapLimitHelper(NextPullData->V)))
			{
				const int32 RenderInterpErrorCorrectionDurationTicks = FMath::FloorToInt32(ErrorCorrectionDuration / AsyncFixedTimeStep); // Convert duration from seconds to simulation ticks

				InterpData->AccumulateErrorXR(Error->ErrorX, Error->ErrorR, SolverSyncTimestamp, RenderInterpErrorCorrectionDurationTicks);
			}
		}
		const bool bHasInterpolationData = InterpData != nullptr;

		SyncedData_External.bIsAnchored = CurrentPullData.bIsAnchored;
		SyncedData_External.bDidSyncGeometry = false;
		SyncedData_External.ChildParticles.Empty(CurrentPullData.ChildParticles.Num());

		// I question the need to do this individual copy one by one...maybe we should've used the same type for the synced data.
		for (const FDirtyClusterUnionParticleData& InData : CurrentPullData.ChildParticles)
		{
			FClusterUnionChildData ConvertedData;
			ConvertedData.ParticleIdx = InData.ParticleIdx;
			ConvertedData.ChildToParent = InData.ChildToParent;
			ConvertedData.Proxy = InData.Proxy;
			ConvertedData.CachedOwner = InData.CachedOwner;
			ConvertedData.BoneId = InData.BoneId;
			SyncedData_External.ChildParticles.Add(ConvertedData);
		}

		if (CurrentPullData.Geometry)
		{
			Particle_External->SetGeometry(CurrentPullData.Geometry);
			SyncedData_External.bDidSyncGeometry = true;
		}

		FReal M = FReal(CurrentPullData.Mass);
		FReal InvM = (M > 0.0) ? (1.0 / M) : 0.0;
		Particle_External->SetM(M, /*bInvalidate=*/false);
		Particle_External->SetInvM(InvM, /*bInvalidate=*/false);

		FVec3 I = FVec3(CurrentPullData.Inertia);
		FVec3 InvI = (!I.IsZero()) ? FVec3(1) / I : FVec3(0);
		Particle_External->SetI(I, /*bInvalidate=*/false);
		Particle_External->SetInvI(InvI, /*bInvalidate=*/false);

		Particle_External->SetObjectState(CurrentPullData.ObjectState, true, /*bInvalidate=*/false);

		const FShapesArray& ShapeArray = Particle_External->ShapesArray();
		for (int32 ShapeIndex = 0; ShapeIndex < ShapeArray.Num(); ++ShapeIndex)
		{
			if (const TUniquePtr<Chaos::FPerShapeData>& ShapeData = ShapeArray[ShapeIndex])
			{
				if (CurrentPullData.CollisionData.IsValidIndex(ShapeIndex))
				{
					ShapeData->SetCollisionData(CurrentPullData.CollisionData[ShapeIndex]);
				}
				if (CurrentPullData.ShapeFilterData.IsValidIndex(ShapeIndex))
				{
					ShapeData->SetShapeFilterData(CurrentPullData.ShapeFilterData[ShapeIndex]);
				}

				if (CurrentPullData.ShapeInstanceData.IsValidIndex(ShapeIndex))
				{
					ShapeData->SetFilterInstanceData(CurrentPullData.ShapeInstanceData[ShapeIndex]);
				}
			}
		}

		if (NextPullData)
		{
			const Chaos::FRealSingle LerpAlpha = RenderInterpolationCVars::bSnapToLatest ? 1.0f : *Alpha;

			// This is the same as in the SingleParticlePhysicsProxy.
			auto LerpHelper = [SolverSyncTimestamp](const auto& Prev, const auto& OverwriteProperty) -> const auto*
			{
				//if overwrite is in the future, do nothing
				//if overwrite is on this step, we want to interpolate from overwrite to the result of the frame that consumed the overwrite
				//if overwrite is in the past, just do normal interpolation

				//this is nested because otherwise compiler can't figure out the type of nullptr with an auto return type
				return OverwriteProperty.Timestamp <= SolverSyncTimestamp ? (OverwriteProperty.Timestamp < SolverSyncTimestamp ? &Prev : &OverwriteProperty.Value) : nullptr;
			};

			if (bHasInterpolationData)
			{
				InterpData->UpdateError(SolverSyncTimestamp, AsyncFixedTimeStep);
			}
			const bool bIsReplicationErrorSmoothing = bHasInterpolationData ? InterpData->IsErrorSmoothing() : false;
			bool DirectionalDecayPerformed = false;

			if (const FVec3* Prev = LerpHelper(PullData.X, ProxyTimestamp->OverWriteX))
			{
				if (bHasInterpolationData)
				{
					DirectionalDecayPerformed = InterpData->DirectionalDecay(NextPullData->X - *Prev, RenderInterpolationCVars::RenderInterpErrorDirectionalDecayMultiplier);
				}

				const FVec3 NewX = bIsReplicationErrorSmoothing ?
					FMath::Lerp(*Prev, NextPullData->X, LerpAlpha) + InterpData->GetErrorX(LerpAlpha) :
					FMath::Lerp(*Prev, NextPullData->X, LerpAlpha);

				Particle_External->SetX(NewX, false);
			}

			if (const FQuat* Prev = LerpHelper(PullData.R, ProxyTimestamp->OverWriteR))
			{
				const FQuat NewR = bIsReplicationErrorSmoothing ? 
					FMath::Lerp(*Prev, NextPullData->R, LerpAlpha) * InterpData->GetErrorR(LerpAlpha) :
					FMath::Lerp(*Prev, NextPullData->R, LerpAlpha); // Add rotational error offset in local space

				Particle_External->SetR(NewR, false);
			}

			if (const FVec3* Prev = LerpHelper(PullData.V, ProxyTimestamp->OverWriteV))
			{
				const FVec3 NewV = FMath::Lerp(*Prev, NextPullData->V, LerpAlpha);
				Particle_External->SetV(NewV, false);
			}

			if (const FVec3* Prev = LerpHelper(PullData.W, ProxyTimestamp->OverWriteW))
			{
				const FVec3 NewW = FMath::Lerp(*Prev, NextPullData->W, LerpAlpha);
				Particle_External->SetW(NewW, false);
			}

#if CHAOS_DEBUG_DRAW
			if (RenderInterpolationCVars::bRenderInterpDebugDraw)
			{
				const FVector ZOffset = FVector(0, 0, RenderInterpolationCVars::RenderInterpDebugDrawZOffset);
				Chaos::FDebugDrawQueue::GetInstance().DrawDebugBox(ZOffset + NextPullData->X, FVector(2, 1, 1), NextPullData->R, FColor::Yellow, false, 5.f, 0, 0.5f);
				Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow(ZOffset + PullData.X, ZOffset + NextPullData->X, 0.5f, FColor::Yellow, false, 5.0f, 0, 0.5f);
				Chaos::FDebugDrawQueue::GetInstance().DrawDebugBox(ZOffset + Particle_External->GetX(), FVector(2, 1, 1), Particle_External->R(), DirectionalDecayPerformed ? FColor::Cyan : FColor::Green, false, 5.f, 0, 0.5f);

				if (bIsReplicationErrorSmoothing)
				{
					if (Error)
					{
						Chaos::FDebugDrawQueue::GetInstance().DrawDebugBox(ZOffset + PullData.X, FVector(4, 2, 2), PullData.R, FColor::Red, false, 5.f, 0, 0.5f);
						Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow(ZOffset + (PullData.X + Error->ErrorX), ZOffset + PullData.X, 1, FColor::Red, false, 5.0f, 0, 0.5f);
					}

					Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow(ZOffset + (Particle_External->GetX() - InterpData->GetErrorX(LerpAlpha)), ZOffset + Particle_External->GetX(), 1, FColor::Blue, false, 5.0f, 0, 0.5f);
				}
			}
#endif // CHAOS_DEBUG_DRAW
		}
		else
		{
			if (SolverSyncTimestamp >= ProxyTimestamp->OverWriteX.Timestamp)
			{
				Particle_External->SetX(PullData.X, false);
			}

			if (SolverSyncTimestamp >= ProxyTimestamp->OverWriteR.Timestamp)
			{
				Particle_External->SetR(PullData.R, false);
			}

			if (SolverSyncTimestamp >= ProxyTimestamp->OverWriteV.Timestamp)
			{
				Particle_External->SetV(PullData.V, false);
			}

			if (SolverSyncTimestamp >= ProxyTimestamp->OverWriteW.Timestamp)
			{
				Particle_External->SetW(PullData.W, false);
			}
		}
		
		// Note: Is their a better way to achieve this? 
		// Don't want to iterate over all children if we don't need to so using flag
		if (CVars::bChaosEnableOverrideSingleParticleTypeToCluster)
		{
			for (const FDirtyClusterUnionParticleData& InData : CurrentPullData.ChildParticles)
			{
				// SingleParticleProxies will not render in the correct location if they are still part on an intact cluster
				// so update the transform of these 'kinematic' child particles
				if (InData.Proxy && InData.Proxy->GetType() == EPhysicsProxyType::SingleParticleProxy)
				{
					if (Chaos::FSingleParticlePhysicsProxy* SingleParticleProxy = static_cast<Chaos::FSingleParticlePhysicsProxy*>(InData.Proxy))
					{
						Chaos::FRigidBodyHandle_External& Body_External = SingleParticleProxy->GetGameThreadAPI();
						FTransform ClusterTransform(Particle_External->R(), Particle_External->X());
						FTransform ChildWorldTransform = InData.ChildToParent * ClusterTransform;
						Body_External.SetX(ChildWorldTransform.GetLocation());
						Body_External.SetR(ChildWorldTransform.GetRotation());
					}
				}
			}
		}
			
		Particle_External->UpdateShapeBounds();
		return true;
	}

	DECLARE_CYCLE_STAT(TEXT("FClusterUnionPhysicsProxy::BufferPhysicsResults_Internal"), STAT_ClusterUnionPhysicsProxyBufferPhysicsResultsInternal, STATGROUP_Chaos);
	void FClusterUnionPhysicsProxy::BufferPhysicsResults_Internal(FDirtyClusterUnionData& BufferData)
	{
		SCOPE_CYCLE_COUNTER(STAT_ClusterUnionPhysicsProxyBufferPhysicsResultsInternal);
		BufferPhysicsResultsImp(this, Particle_Internal, BufferData);
	
		FPBDRigidsEvolutionGBF& Evolution = *static_cast<FPBDRigidsSolver*>(Solver)->GetEvolution();
		FClusterUnionManager& ClusterUnionManager = Evolution.GetRigidClustering().GetClusterUnionManager();

		if (FClusterUnion* ClusterUnion = ClusterUnionManager.FindClusterUnion(ClusterUnionIndex))
		{
			BufferData.ChildParticles.Empty(ClusterUnion->ChildParticles.Num());

			for (FPBDRigidParticleHandle* Particle : ClusterUnion->ChildParticles)
			{
				if (!ensure(Particle))
				{
					continue;
				}

				if (IPhysicsProxyBase* Proxy = Particle->PhysicsProxy())
				{
					FDirtyClusterUnionParticleData Data;
					Data.ParticleIdx = Particle->UniqueIdx();
					if (FPBDRigidClusteredParticleHandle* ClusteredParticle = Particle->CastToClustered())
					{
						Data.ChildToParent = ClusteredParticle->ChildToParent();
					}
					else
					{
						Data.ChildToParent = FRigidTransform3::Identity;
					}
					Data.Proxy = Proxy;
					Data.CachedOwner = reinterpret_cast<void*>(Proxy->GetOwner());
					
					if (Proxy->GetType() == EPhysicsProxyType::GeometryCollectionType)
					{
						// A bit hacky, but the only way we can communicate what particle we care about back to the cluster union on the GT.
						FGeometryCollectionPhysicsProxy* GCProxy = static_cast<FGeometryCollectionPhysicsProxy*>(Proxy);
						Data.BoneId = GCProxy->GetTransformGroupIndexFromHandle(Particle);
					}
					else if (Proxy->GetType() == EPhysicsProxyType::SingleParticleProxy)
					{
						// There's no way to reconstitute the BoneID! BoneID is used to match the particle on the client, but this makes no sense when dealing with SingleParticle
						Data.BoneId = BufferData.ChildParticles.Num(); // This could be wrong, but seems to work for now!
					}

					BufferData.ChildParticles.Add(Data);
				}
			}

			if (ClusterUnion->bGeometryModified && !ClusterUnion->ChildParticles.IsEmpty())
			{
				if(FImplicitObjectUnion* AsUnion = ClusterUnion->Geometry->AsA<FImplicitObjectUnion>();
				   GShallowCopyClusterUnionGeometryOnUpdate && AsUnion)
				{
					// Shallow copy the root union for the GT update
					// This ensures the GT has a snapshot of the union as it is now for this results data. The PT is free to
					// continue modifying its geometry without potentially disrupting GT reads. Internally we don't want
					// to duplicate all the geometry - we essentially just need the list of objects in the root union to
					// point to the correct internal geometries, but have a separate list on GT and PT.
					TArray<FImplicitObjectPtr> ObjectsCopy = AsUnion->GetObjects();
					BufferData.Geometry = MakeImplicitObjectPtr<FImplicitObjectUnion>(MoveTemp(ObjectsCopy));
				}
				else
				{
					// Fallback to deep copy geometry hierarchy
					BufferData.Geometry = ClusterUnion->Geometry->DeepCopyGeometry();
				}

				if (FImplicitObjectUnion* Union = BufferData.Geometry->AsA<FImplicitObjectUnion>())
				{
					Union->SetLockedRecursive(true);
				}

				ClusterUnion->bGeometryModified = false;
			}
		}
	}

	DECLARE_CYCLE_STAT(TEXT("FClusterUnionPhysicsProxy::BufferPhysicsResults_External"), STAT_ClusterUnionPhysicsProxyBufferPhysicsResultsExternal, STATGROUP_Chaos);
	void FClusterUnionPhysicsProxy::BufferPhysicsResults_External(FDirtyClusterUnionData& BufferData)
	{
		SCOPE_CYCLE_COUNTER(STAT_ClusterUnionPhysicsProxyBufferPhysicsResultsExternal);
		BufferPhysicsResultsImp(this, Particle_External.Get(), BufferData);
		BufferData.bIsAnchored = SyncedData_External.bIsAnchored;

		BufferData.ChildParticles.Empty(SyncedData_External.ChildParticles.Num());
		for (const FClusterUnionChildData& Data : SyncedData_External.ChildParticles)
		{
			FDirtyClusterUnionParticleData ConvertedData;
			ConvertedData.ParticleIdx = Data.ParticleIdx;
			ConvertedData.ChildToParent = Data.ChildToParent;
			ConvertedData.Proxy = Data.Proxy;
			ConvertedData.CachedOwner = Data.CachedOwner;
			ConvertedData.BoneId = Data.BoneId;
			BufferData.ChildParticles.Add(ConvertedData);
		}
	}

	DECLARE_CYCLE_STAT(TEXT("FClusterUnionPhysicsProxy::SyncRemoteData"), STAT_ClusterUnionPhysicsProxySyncRemoteData, STATGROUP_Chaos);
	void FClusterUnionPhysicsProxy::SyncRemoteData(FDirtyPropertiesManager& Manager, int32 DataIdx, FDirtyChaosProperties& RemoteData) const
	{
		SCOPE_CYCLE_COUNTER(STAT_ClusterUnionPhysicsProxySyncRemoteData);
		if (!ensure(Particle_External))
		{
			return;
		}

		// This is similar to TGeometryParticle::SyncRemoteData except it puts it into the cluster properties.
		RemoteData.SetParticleBufferType(Particle_External->Type);

		// We need to modify the dirty flags to remove the non cluster properties to be 100% safe.
		FDirtyChaosPropertyFlags DirtyFlags = Particle_External->DirtyFlags();

		// We need to suppress V501 in PVS Studio since it's a false positive warning in this case. This is actually necessary for the codegen here.
//-V:CHAOS_PROPERTY:501
#define CHAOS_PROPERTY(PropName, Type, ProxyType) if constexpr (ProxyType != EPhysicsProxyType::ClusterUnionProxy) { DirtyFlags.MarkClean(EChaosPropertyFlags::PropName); }
#include "Chaos/ParticleProperties.inl" //-V:CHAOS_PROPERTY:501
#undef CHAOS_PROPERTY

		RemoteData.SetFlags(DirtyFlags);

		// SyncRemote will check the dirty flags and will skip the change in value if the dirty flag is not actually set.
		RemoteData.SyncRemote<FParticlePositionRotation, EChaosProperty::ClusterXR>(Manager, DataIdx, Particle_External->XR());
		RemoteData.SyncRemote<FParticleVelocities, EChaosProperty::ClusterVelocities>(Manager, DataIdx, Particle_External->Velocities());
	}

	void FClusterUnionPhysicsProxy::ClearAccumulatedData()
	{
		if (!ensure(Particle_External))
		{
			return;
		}

		Particle_External->ClearDirtyFlags();
	}

	void FClusterUnionPhysicsProxy::SetXR_External(const FVector& X, const FQuat& R)
	{
		if (!Solver || !ensure(Particle_External))
		{
			return;
		}

		Particle_External->SetX(X, false);
		Particle_External->SetR(R, false);
		Particle_External->ForceDirty(EChaosPropertyFlags::ClusterXR);

		FClusterUnionProxyTimestamp& SyncTS = GetSyncTimestampAs<FClusterUnionProxyTimestamp>();
		SyncTS.OverWriteX.Set(GetSolverSyncTimestamp_External(), X);
		SyncTS.OverWriteR.Set(GetSolverSyncTimestamp_External(), R);

		Solver->AddDirtyProxy(this);
	}

	void FClusterUnionPhysicsProxy::SetLinearVelocity_External(const FVector& V)
	{
		if (!Solver || !ensure(Particle_External))
		{
			return;
		}

		Particle_External->SetV(V, false);
		Particle_External->ForceDirty(EChaosPropertyFlags::ClusterVelocities);

		FClusterUnionProxyTimestamp& SyncTS = GetSyncTimestampAs<FClusterUnionProxyTimestamp>();
		SyncTS.OverWriteV.Set(GetSolverSyncTimestamp_External(), V);

		Solver->AddDirtyProxy(this);
	}

	void FClusterUnionPhysicsProxy::SetAngularVelocity_External(const FVector& W)
	{
		if (!ensure(Particle_External))
		{
			return;
		}

		Particle_External->SetW(W, false);
		Particle_External->ForceDirty(EChaosPropertyFlags::ClusterVelocities);

		FClusterUnionProxyTimestamp& SyncTS = GetSyncTimestampAs<FClusterUnionProxyTimestamp>();
		SyncTS.OverWriteW.Set(GetSolverSyncTimestamp_External(), W);

		Solver->AddDirtyProxy(this);
	}

	void FClusterUnionPhysicsProxy::SetChildToParent_External(FPhysicsObjectHandle Child, const FTransform& RelativeTransform, bool bLock)
	{
		if (ensure(Child != nullptr))
		{
			BulkSetChildToParent_External({ Child }, { RelativeTransform }, bLock);
		}
	}

	void FClusterUnionPhysicsProxy::BulkSetChildToParent_External(const TArray<FPhysicsObjectHandle>& Objects, const TArray<FTransform>& Transforms, bool bLock)
	{
		if (!Solver || !ensure(Particle_External) || !ensure(Objects.Num() == Transforms.Num()))
		{
			return;
		}

		Solver->EnqueueCommandImmediate(
			[this, Objects, Transforms, bLock]() mutable
			{
				FReadPhysicsObjectInterface_Internal Interface = FPhysicsObjectInternalInterface::GetRead();
				TArray<FPBDRigidParticleHandle*> Particles = Interface.GetAllRigidParticles(Objects, /* bIncludeNulls */ true);
				if (ensure(Particles.Num() == Objects.Num()))
				{
					FPBDRigidsEvolutionGBF& Evolution = *static_cast<FPBDRigidsSolver*>(Solver)->GetEvolution();
					FClusterUnionManager& ClusterUnionManager = Evolution.GetRigidClustering().GetClusterUnionManager();
					ClusterUnionManager.UpdateClusterUnionParticlesChildToParent(ClusterUnionIndex, Particles, Transforms, bLock);
				}
			}
		);
	}

	void FClusterUnionPhysicsProxy::SetEnableStrainOnCollision_External(bool bEnable)
	{
		if (Solver && ensure(Particle_External))
		{
			Solver->EnqueueCommandImmediate(
				[this, bEnable]() mutable
				{
					bEnableStrainOnCollision_Internal = bEnable;
				}
			);
		}
	}

	void FClusterUnionPhysicsProxy::ChangeMainParticleStatus_External(const TArray<FPhysicsObjectHandle>& Objects, bool bIsMain)
	{
		if (!Solver)
		{
			return;
		}

		Solver->EnqueueCommandImmediate(
			[this, Objects, bIsMain]() mutable
			{
				FReadPhysicsObjectInterface_Internal Interface = FPhysicsObjectInternalInterface::GetRead();
				TArray<FPBDRigidParticleHandle*> Particles = Interface.GetAllRigidParticles(Objects);

				FPBDRigidsEvolutionGBF& Evolution = *static_cast<FPBDRigidsSolver*>(Solver)->GetEvolution();
				FClusterUnionManager& ClusterUnionManager = Evolution.GetRigidClustering().GetClusterUnionManager();

				if (FClusterUnion* Union = ClusterUnionManager.FindClusterUnion(ClusterUnionIndex))
				{
					for (FPBDRigidParticleHandle* Particle : Particles)
					{
						if (FClusterUnionParticleProperties* Props = Union->ChildProperties.Find(Particle))
						{
							Props->bIsAuxiliaryParticle = !bIsMain;
						}
					}
				}
			}
		);
	}
}
