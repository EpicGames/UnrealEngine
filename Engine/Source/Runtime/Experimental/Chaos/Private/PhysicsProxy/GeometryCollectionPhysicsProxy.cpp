// Copyright Epic Games, Inc. All Rights Reserved.
#include "PhysicsProxy/GeometryCollectionPhysicsProxy.h"
#include "PhysicsProxy/FieldSystemProxyHelper.h"

#include "PhysicsSolver.h"
#include "ChaosStats.h"
#include "Chaos/ArrayCollectionArray.h"
#include "Chaos/BVHParticles.h"
#include "Chaos/Transform.h"
#include "Chaos/ParallelFor.h"
#include "Chaos/GeometryParticles.h"
#include "Chaos/Particles.h"
#include "Chaos/TriangleMesh.h"
#include "Chaos/MassProperties.h"
#include "ChaosSolversModule.h"
#include "Chaos/PBDCollisionConstraintsUtil.h"
#include "Chaos/PerParticleGravity.h"
#include "Chaos/PhysicsObjectInternalInterface.h"
#include "Chaos/ImplicitObject.h"
#include "Chaos/ImplicitObjectScaled.h"
#include "Chaos/ImplicitObjectUnion.h"
#include "Chaos/Convex.h"
#include "Chaos/Serializable.h"
#include "Chaos/ErrorReporter.h"
#include "Chaos/PBDRigidClustering.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/GeometryCollectionUtility.h"
#include "GeometryCollection/GeometryCollectionSizeSpecificUtility.h"
#include "GeometryCollection/GeometryCollectionSimulationTypes.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/ManagedArrayAccessor.h"
#include "Modules/ModuleManager.h"
#include "Chaos/PullPhysicsDataImp.h"
#include "Chaos/PBDRigidsEvolution.h"
#include "Chaos/PhysicsObjectInternal.h"
#include "Chaos/PhysicsObjectInterface.h"
#include "GeometryCollection/Facades/CollectionAnchoringFacade.h"
#include "GeometryCollection/Facades/CollectionConnectionGraphFacade.h"
#include "Chaos/DebugDrawQueue.h"

#define GC_PHYSICSPROXY_CHECK_FOR_NAN_ENABLED 0

#if GC_PHYSICSPROXY_CHECK_FOR_NAN_ENABLED
#define GC_PHYSICSPROXY_CHECK_FOR_NAN(Vec) ensure(!Vec.ContainsNaN())
#else
#define GC_PHYSICSPROXY_CHECK_FOR_NAN(Vec)
#endif

namespace
{
	const FName MassToLocalAttributeName = "MassToLocal";
	const FName MassAttributeName = "Mass";
	const FName InertiaTensorAttributeName = "InertiaTensor";
	const FName LevelAttributeName = "Level";
}

namespace Chaos{
	extern int32 AccelerationStructureSplitStaticAndDynamic;
	extern int32 AccelerationStructureIsolateQueryOnlyObjects;
}

float CollisionParticlesPerObjectFractionDefault = 1.0f;
FAutoConsoleVariableRef CVarCollisionParticlesPerObjectFractionDefault(
	TEXT("p.CollisionParticlesPerObjectFractionDefault"), 
	CollisionParticlesPerObjectFractionDefault, 
	TEXT("Fraction of verts"));

bool DisableGeometryCollectionGravity = false;
FAutoConsoleVariableRef CVarGeometryCollectionDisableGravity(
	TEXT("p.GeometryCollectionDisableGravity"),
	DisableGeometryCollectionGravity,
	TEXT("Disable gravity for geometry collections"));

bool GeometryCollectionCollideAll = false;
FAutoConsoleVariableRef CVarGeometryCollectionCollideAll(
	TEXT("p.GeometryCollectionCollideAll"),
	GeometryCollectionCollideAll,
	TEXT("Bypass the collision matrix and make geometry collections collide against everything"));


bool bGeometryCollectionEnabledNestedChildTransformUpdates = true;
FAutoConsoleVariableRef CVarEnabledNestedChildTransformUpdates(
	TEXT("p.GeometryCollection.EnabledNestedChildTransformUpdates"),
	bGeometryCollectionEnabledNestedChildTransformUpdates,
	TEXT("Enable updates for driven, disabled, child bodies. Used for line trace results against geometry collections.[def: true]"));

bool bGeometryCollectionAlwaysGenerateGTCollisionForClusters = true;
FAutoConsoleVariableRef CVarGeometryCollectionAlwaysGenerateGTCollisionForClusters(
	TEXT("p.GeometryCollection.AlwaysGenerateGTCollisionForClusters"),
	bGeometryCollectionAlwaysGenerateGTCollisionForClusters,
	TEXT("When enabled, always generate a game thread side collision for clusters.[def: true]"));

bool bGeometryCollectionAlwaysGenerateConnectionGraph = false;
FAutoConsoleVariableRef CVarGeometryCollectionAlwaysGenerateConnectionGraph(
	TEXT("p.GeometryCollection.AlwaysGenerateConnectionGraph"),
	bGeometryCollectionAlwaysGenerateConnectionGraph,
	TEXT("When enabled, always  generate the cluster's connection graph instead of using the rest collection stored one - Note: this should only be used for troubleshooting.[def: false]"));

int32 GeometryCollectionAreaBasedDamageThresholdMode = 0;
FAutoConsoleVariableRef CVarGeometryCollectionAreaBasedDamageThresholdMode(
	TEXT("p.GeometryCollection.AreaBasedDamageThresholdMode"),
	GeometryCollectionAreaBasedDamageThresholdMode,
	TEXT("Area based damage threshold computation mode (0: sum of areas | 1: max of areas | 2: min of areas | 3: average of areas) [def: 0]"));

int32 GeometryCollectionLocalInertiaDropOffDiagonalTerms = 0;
FAutoConsoleVariableRef CVarGeometryCollectionLocalInertiaDropOffDiagonalTerms(
	TEXT("p.GeometryCollection.LocalInertiaDropOffDiagonalTerms"),
	GeometryCollectionLocalInertiaDropOffDiagonalTerms,
	TEXT("When true, force diagonal inertia for GCs in their local space by simply dropping off-diagonal terms"));

float GeometryCollectionTransformTolerance = 0.001f;
FAutoConsoleVariableRef CVarGeometryCollectionTransformTolerance(
	TEXT("p.GeometryCollection.TransformTolerance"),
	GeometryCollectionTransformTolerance,
	TEXT("Tolerance to detect if a transform has changed"));

float GeometryCollectionPositionUpdateTolerance = UE_KINDA_SMALL_NUMBER;
FAutoConsoleVariableRef CVarGeometryCollectionPositionUpdateTolerance(
	TEXT("p.GeometryCollection.PositionUpdateTolerance"),
	GeometryCollectionPositionUpdateTolerance,
	TEXT("Tolerance to detect if particle position has changed has changed when syncing PT to GT"));

float GeometryCollectionRotationUpdateTolerance = UE_KINDA_SMALL_NUMBER;
FAutoConsoleVariableRef CVarGeometryCollectionRotationUpdateTolerance(
	TEXT("p.GeometryCollection.RotationUpdateTolerance"),
	GeometryCollectionRotationUpdateTolerance,
	TEXT("Tolerance to detect if particle rotation has changed has changed when syncing PT to GT"));

float GeometryCollectionLinearVelocityUpdateTolerance = UE_KINDA_SMALL_NUMBER;
FAutoConsoleVariableRef CVarGeometryCollectionLinearVelocityUpdateTolerance(
	TEXT("p.GeometryCollection.LinearVelocityUpdateTolerance"),
	GeometryCollectionLinearVelocityUpdateTolerance,
	TEXT("Tolerance to detect if particle linear velocity has changed has changed when syncing PT to GT"));

float GeometryCollectionAngularVelocityUpdateTolerance = UE_KINDA_SMALL_NUMBER;
FAutoConsoleVariableRef CVarGeometryCollectionAngularVelocityUpdateTolerance(
	TEXT("p.GeometryCollection.AngularVelocityUpdateTolerance"),
	GeometryCollectionAngularVelocityUpdateTolerance,
	TEXT("Tolerance to detect if particle angular velocity has changed has changed when syncing PT to GT"));

bool bGeometryCollectionUseRootBrokenFlag = true;
FAutoConsoleVariableRef CVarGeometryCollectionUseRootBrokenFlag(
	TEXT("p.GeometryCollection.UseRootBrokenFlag"),
	bGeometryCollectionUseRootBrokenFlag,
	TEXT("If enabled, check if the root transform is broken in the proxy and disable the GT particle if so. Should be enabled - cvar is a failsafe to revert behaviour"));

bool bPropagateInternalClusterDisableFlagToChildren = true;
FAutoConsoleVariableRef CVarPropagateInternalClusterDisableFlagToChildren(
	TEXT("p.GeometryCollection.PropagateInternalClusterDisableFlagToChildren"),
	bPropagateInternalClusterDisableFlagToChildren,
	TEXT("If enabled, disabled internal clusters will propagate their disabled flag to their children when buffering instead of implicitly activating the children."));
	
bool bGeometryCollectionScaleClusterGeometry = true;
FAutoConsoleVariableRef CVarGeometryCollectionScaleClusterGeometry(
	TEXT("p.GeometryCollection.ScaleClusterGeometry"),
	bGeometryCollectionScaleClusterGeometry,
	TEXT("If enabled, update the cluster geometry if the scale has changed"));

bool bEmptyUniqueIndicesOnGT = false;
FAutoConsoleVariableRef CVarEmptyUniqueIndicesOnGT(
	TEXT("p.GeometryCollection.EmptyUniqueIndicesOnGT"),
	bEmptyUniqueIndicesOnGT,
	TEXT("If enabled, immediately free particle unique indices after particle initialisation"));

bool bUpdateGTParticlesOnWorldTransformUpdate = false;
FAutoConsoleVariableRef CVarUpdateGTParticlesOnWorldTransformUpdate(
	TEXT("p.GeometryCollection.UpdateGTParticlesOnWorldTransformUpdate"),
	bUpdateGTParticlesOnWorldTransformUpdate,
	TEXT("If enabled, SetWorldTransform_External will immediately update the GT particle to reflect the new transform instead of waiting for a full GT -> PT -> GT sync to do so"));

enum EOverrideGCCollisionSetupForTraces
{
	GCCSFT_Property   = -1,  // Default: do what property says
	GCCSFT_ForceSM    =  0,  // Force the use of SM collision
	GCCSFT_ForceGC    =  1,  // Force the use of GC collision
};

#if !UE_BUILD_SHIPPING

int32 ForceOverrideGCCollisionSetupForTraces = GCCSFT_Property;
FAutoConsoleVariableRef CVarForceOverrideGCCollisionSetupForTraces(
	TEXT("p.GeometryCollection.ForceOverrideGCCollisionSetupForTraces"),
	ForceOverrideGCCollisionSetupForTraces,
	TEXT("Force the usage of a specific type of collision for traces on the game thread when creating new GC physics representations (-1: use the value of the property | 0: force to use SM collision | 1: force to use GC collision) [def: -1]"));

#else

constexpr int32 ForceOverrideGCCollisionSetupForTraces = GCCSFT_Property;

#endif // !UE_BUILD_SHIPPING


DEFINE_LOG_CATEGORY_STATIC(UGCC_LOG, Error, All);

static const FSharedSimulationSizeSpecificData& GetSizeSpecificData(const TArray<FSharedSimulationSizeSpecificData>& SizeSpecificData, const FGeometryCollection& RestCollection, const int32 TransformIndex, const FBox& BoundingBox);
static Chaos::FImplicitObjectPtr MakeTransformImplicitObject(const Chaos::FImplicitObject& ImplicitObject, const Chaos::FRigidTransform3& Transform);

//==============================================================================
// FGeometryCollectionResults
//==============================================================================

FGeometryCollectionResults::FGeometryCollectionResults()
	: IsObjectDynamic(false)
	, IsObjectLoading(false)
	, IsRootBroken(false)
{}

void FGeometryCollectionResults::Reset()
{
	SolverDt = 0.0f;

	ModifiedTransformIndices.Reset();
	States.Reset();
	Positions.Reset();
	Velocities.Reset();

	IsObjectDynamic = false;
	IsObjectLoading = false;
	IsRootBroken = false;
}

//==============================================================================
// FGeometryCollectionPhysicsProxy helper functions
//==============================================================================

template<typename TLambda>
void ExecuteOnPhysicsThread(FGeometryCollectionPhysicsProxy& Proxy, TLambda&& Lambda)
{
	if (Chaos::FPhysicsSolver* RBDSolver = Proxy.GetSolver<Chaos::FPhysicsSolver>())
	{
		RBDSolver->EnqueueCommandImmediate(Lambda);
	}
}

TUniquePtr<Chaos::FTriangleMesh> CreateTriangleMesh(
	const int32 FaceStart,
	const int32 FaceCount, 
	const TManagedArray<bool>& Visible, 
	const TManagedArray<FIntVector>& Indices,
	bool bRotateWinding)
{
	TArray<Chaos::TVector<int32, 3>> Faces;
	Faces.Reserve(FaceCount);
	
	const int32 FaceEnd = FaceStart + FaceCount;
	for (int Idx = FaceStart; Idx < FaceEnd; ++Idx)
	{
		// Note: This function used to cull small triangles.  As one of the purposes 
		// of the tri mesh this function creates is for level set rasterization, we 
		// don't want to do that.  Keep the mesh intact, which hopefully is water tight.
		if (Visible[Idx])
		{
			const FIntVector& Tri = Indices[Idx];

			if(bRotateWinding)
			{
				Faces.Add(Chaos::TVector<int32, 3>(Tri.Z, Tri.Y, Tri.X));
			}
			else
			{
				Faces.Add(Chaos::TVector<int32, 3>(Tri.X, Tri.Y, Tri.Z));
			}
		}
	}
	return MakeUnique<Chaos::FTriangleMesh>(MoveTemp(Faces)); // Culls geometrically degenerate faces
}

TArray<int32> ComputeTransformToGeometryMap(const FGeometryCollection& Collection)
{
	const int32 NumTransforms = Collection.NumElements(FGeometryCollection::TransformGroup);
	const int32 NumGeometries = Collection.NumElements(FGeometryCollection::GeometryGroup);
	const TManagedArray<int32>& TransformIndex = Collection.TransformIndex;

	TArray<int32> TransformToGeometryMap;
	TransformToGeometryMap.AddUninitialized(NumTransforms);
	for(int32 GeometryIndex = 0; GeometryIndex < NumGeometries; ++GeometryIndex)
	{
		const int32 TransformGroupIndex = TransformIndex[GeometryIndex];
		TransformToGeometryMap[TransformGroupIndex] = GeometryIndex;
	}

	return TransformToGeometryMap;
}

static float AreaFromBoundingBoxOverlap(const Chaos::FAABB3& BoxA, const Chaos::FAABB3& BoxB)
{
	// if the two box don't overlap, we'll get an inside out box 
	// but we are still using it to compute the area as an approximation
	const Chaos::FAABB3 OverlapBox = BoxA.GetIntersection(BoxB);

	const Chaos::FVec3 Extents = OverlapBox.Extents();
	const Chaos::FVec3 CenterToCenterNormal = (BoxA.GetCenter() - BoxB.GetCenter()).GetSafeNormal();

	const Chaos::FVec3 AreaPerAxis{
		FMath::Abs(Extents.Y * Extents.Z),
		FMath::Abs(Extents.X * Extents.Z),
		FMath::Abs(Extents.X * Extents.Y)
	};

	// weight the area by the center to center normal
	const Chaos::FReal Area = FMath::Abs(AreaPerAxis.Dot(CenterToCenterNormal));

	return static_cast<float>(Area);
}

void SetImplicitToPTParticles(Chaos::TPBDRigidParticleHandle<Chaos::FReal, 3>* Handle,
	const FSharedSimulationParameters& SharedParams,
	const FCollisionStructureManager::FSimplicial* Simplicial,
	Chaos::FImplicitObjectPtr Implicit,
	const Chaos::Filter::FShapeFilterData& ShapeFilterDataIn,
	const Chaos::Filter::FInstanceData& ShapeInstanceDataIn,
	const FTransform& WorldTransform,
	float CollisionParticlesPerObjectFraction)
{
	// @todo(GCCollisionShapes) : add support for multiple shapes, currently just one. 
	FCollectionCollisionTypeData SingleSupportedCollisionTypeData = FCollectionCollisionTypeData();
	if (SharedParams.SizeSpecificData.Num() && SharedParams.SizeSpecificData[0].CollisionShapesData.Num())
	{
		SingleSupportedCollisionTypeData = SharedParams.SizeSpecificData[0].CollisionShapesData[0];
	}
	const FVector Scale = WorldTransform.GetScale3D();
	if (Implicit)	//todo(ocohen): this is only needed for cases where clusters have no proxy. Kind of gross though, should refactor
	{
		auto DeepCopyImplicit = [&Scale](Chaos::FImplicitObjectPtr ImplicitToCopy) -> Chaos::FImplicitObjectPtr
		{
			if (Scale.Equals(FVector::OneVector))
			{
				return ImplicitToCopy->DeepCopyGeometry();
			}
			else
			{
				return ImplicitToCopy->DeepCopyGeometryWithScale(Scale);
			}
		};

		Chaos::EImplicitObjectType ImplicitType = Implicit->GetType();
		// Don't copy if it is not a level set and scale is one
		if (SingleSupportedCollisionTypeData.CollisionType != ECollisionTypeEnum::Chaos_Surface_Volumetric &&
			ImplicitType != Chaos::ImplicitObjectType::LevelSet && Scale.Equals(FVector::OneVector))
		{
			Handle->SetGeometry(Implicit);
			Handle->SetLocalBounds(Implicit->BoundingBox());
		}
		else
		{
			Chaos::FImplicitObjectPtr SharedImplicitTS = DeepCopyImplicit(Implicit);
			FCollisionStructureManager::UpdateImplicitFlags(SharedImplicitTS.GetReference(), SingleSupportedCollisionTypeData.CollisionType);
			Handle->SetGeometry(SharedImplicitTS);
			Handle->SetLocalBounds(SharedImplicitTS->BoundingBox());
		}
		Handle->SetHasBounds(true);
		const Chaos::FRigidTransform3 Xf(Handle->GetX(), Handle->GetR());
		Handle->UpdateWorldSpaceState(Xf, Chaos::FVec3(0));
	}

	if (Simplicial && SingleSupportedCollisionTypeData.CollisionType == ECollisionTypeEnum::Chaos_Surface_Volumetric)
	{
		Handle->CollisionParticlesInitIfNeeded();

		TUniquePtr<Chaos::FBVHParticles>& CollisionParticles = Handle->CollisionParticles();
		CollisionParticles.Reset(Simplicial->NewCopy()); // @chaos(optimize) : maybe just move this memory instead. 

		const int32 NumCollisionParticles = CollisionParticles->Size();
		const int32 AdjustedNumCollisionParticles = FMath::TruncToInt(CollisionParticlesPerObjectFraction * (float)NumCollisionParticles);
		int32 CollisionParticlesSize = FMath::Max<int32>(0, FMath::Min<int32>(AdjustedNumCollisionParticles, NumCollisionParticles));
		CollisionParticles->Resize(CollisionParticlesSize); // Truncates! ( particles are already sorted by importance )

		Chaos::FAABB3 ImplicitShapeDomain = Chaos::FAABB3::FullAABB();
		if (Implicit && Implicit->GetType() == Chaos::ImplicitObjectType::LevelSet && Implicit->HasBoundingBox())
		{
			ImplicitShapeDomain = Implicit->BoundingBox();
			ImplicitShapeDomain.Scale(Scale);
		}

		// we need to account for scale and check if the particle is still within its domain
		for (int32 ParticleIndex = 0; ParticleIndex < (int32)CollisionParticles->Size(); ++ParticleIndex)
		{
			CollisionParticles->SetX(ParticleIndex, CollisionParticles->GetX(ParticleIndex) * Scale);

			// Make sure the collision particles are at least in the domain 
			// of the implicit shape.
			ensure(ImplicitShapeDomain.Contains(CollisionParticles->GetX(ParticleIndex)));
		}

		// @todo(remove): IF there is no simplicial we should not be forcing one. 
		if (!CollisionParticles->Size())
		{
			CollisionParticles->AddParticles(1);
			CollisionParticles->SetX(0, Chaos::FVec3(0));
		}
		CollisionParticles->UpdateAccelerationStructures();
	}

	if (GeometryCollectionCollideAll) // cvar
	{
		// Override collision filters and make this body collide with everything.
		int32 CurrShape = 0;
		
		const Chaos::Filter::FShapeFilterData ShapeFilterData = Chaos::Filter::FShapeFilterBuilder::BuildBlockAll(Chaos::EFilterFlags::All);
		for (const TUniquePtr<Chaos::FPerShapeData>& Shape : Handle->ShapesArray())
		{
			Shape->SetSimEnabled(true);
			Shape->SetCollisionTraceType(Chaos::EChaosCollisionTraceFlag::Chaos_CTF_UseDefault);
			//Shape->CollisionTraceType = Chaos::EChaosCollisionTraceFlag::Chaos_CTF_UseSimpleAndComplex;
			Shape->SetShapeFilterData(ShapeFilterData);
		}
	}
	else
	{
		for (const TUniquePtr<Chaos::FPerShapeData>& Shape : Handle->ShapesArray())
		{
			Shape->SetShapeFilterData(ShapeFilterDataIn);
			Shape->SetFilterInstanceData(ShapeInstanceDataIn);
		}
	}
}

DECLARE_CYCLE_STAT(TEXT("FGeometryCollectionPhysicsProxy::PopulateSimulatedParticle"), STAT_PopulateSimulatedParticle, STATGROUP_Chaos);
void PopulateSimulatedParticle(
	Chaos::TPBDRigidParticleHandle<Chaos::FReal,3>* Handle,
	const FSharedSimulationParameters& SharedParams,
	const FCollisionStructureManager::FSimplicial* Simplicial,
	Chaos::FImplicitObjectPtr Implicit,
	const Chaos::Filter::FShapeFilterData& ShapeFilterDataIn,
	const Chaos::Filter::FInstanceData& ShapeInstanceDataIn,
	Chaos::FReal MassIn,
	Chaos::TVec3<Chaos::FRealSingle> InertiaTensorVec,
	const FTransform& WorldTransform, 
	const uint8 DynamicState, 
	const int16 CollisionGroup,
	float CollisionParticlesPerObjectFraction)
{
	SCOPE_CYCLE_COUNTER(STAT_PopulateSimulatedParticle);
	Handle->SetDisabledLowLevel(false);
	Handle->SetX(WorldTransform.GetTranslation());
	Handle->SetVf(Chaos::FVec3f(0.f));
	Handle->SetR(WorldTransform.GetRotation().GetNormalized());
	Handle->SetWf(Chaos::FVec3f(0.f));
	Handle->SetP(Handle->GetX());
	Handle->SetQf(Handle->GetRf());
	Handle->SetCenterOfMass(FVector3f::ZeroVector);
	Handle->SetRotationOfMass(FQuat::Identity);

	//
	// Setup Mass
	//
	{
		Handle->SetObjectStateLowLevel(Chaos::EObjectStateType::Uninitialized);

		if (!CHAOS_ENSURE_MSG(FMath::IsWithinInclusive<Chaos::FReal>(MassIn, SharedParams.MinimumMassClamp, SharedParams.MaximumMassClamp),
			TEXT("Clamped mass[%3.5f] to range [%3.5f,%3.5f]"), MassIn, SharedParams.MinimumMassClamp, SharedParams.MaximumMassClamp))
		{
			MassIn = FMath::Clamp<Chaos::FReal>(MassIn, SharedParams.MinimumMassClamp, SharedParams.MaximumMassClamp);
		}

		if (!CHAOS_ENSURE_MSG(!FMath::IsNaN(InertiaTensorVec[0]) && !FMath::IsNaN(InertiaTensorVec[1]) && !FMath::IsNaN(InertiaTensorVec[2]),
			TEXT("Nan Tensor, reset to unit tesor")))
		{
			InertiaTensorVec = FVector3f(1);
		}
		else if (!CHAOS_ENSURE_MSG(FMath::IsWithinInclusive(InertiaTensorVec[0], SharedParams.MinimumInertiaTensorDiagonalClamp, SharedParams.MaximumInertiaTensorDiagonalClamp)
			&& FMath::IsWithinInclusive(InertiaTensorVec[1], SharedParams.MinimumInertiaTensorDiagonalClamp, SharedParams.MaximumInertiaTensorDiagonalClamp)
			&& FMath::IsWithinInclusive(InertiaTensorVec[2], SharedParams.MinimumInertiaTensorDiagonalClamp, SharedParams.MaximumInertiaTensorDiagonalClamp),
			TEXT("Clamped Inertia tensor[%3.5f,%3.5f,%3.5f]. Clamped each element to [%3.5f, %3.5f,]"), InertiaTensorVec[0], InertiaTensorVec[1], InertiaTensorVec[2],
			SharedParams.MinimumInertiaTensorDiagonalClamp, SharedParams.MaximumInertiaTensorDiagonalClamp))
		{
			InertiaTensorVec[0] = FMath::Clamp(InertiaTensorVec[0], SharedParams.MinimumInertiaTensorDiagonalClamp, SharedParams.MaximumInertiaTensorDiagonalClamp);
			InertiaTensorVec[1] = FMath::Clamp(InertiaTensorVec[1], SharedParams.MinimumInertiaTensorDiagonalClamp, SharedParams.MaximumInertiaTensorDiagonalClamp);
			InertiaTensorVec[2] = FMath::Clamp(InertiaTensorVec[2], SharedParams.MinimumInertiaTensorDiagonalClamp, SharedParams.MaximumInertiaTensorDiagonalClamp);
		}

		Handle->SetM(MassIn);
		Handle->SetI(InertiaTensorVec);
		const Chaos::FReal MassInv = (MassIn > 0.0f) ? 1.0f / MassIn : 0.0f;
		const Chaos::FVec3 InertiaInv = (MassIn > 0.0f) ? Chaos::FVec3(InertiaTensorVec).Reciprocal() : Chaos::FVec3::ZeroVector;
		Handle->SetInvM(MassInv);
		Handle->SetInvI(InertiaInv);
		Handle->SetObjectStateLowLevel(Chaos::EObjectStateType::Dynamic); // this step sets InvM, InvInertia, P, Q
	}

	Handle->SetCollisionGroup(CollisionGroup);

	SetImplicitToPTParticles(Handle, SharedParams, Simplicial, Implicit, ShapeFilterDataIn, ShapeInstanceDataIn, WorldTransform, CollisionParticlesPerObjectFraction);

	//
	//  Manage Object State
	//

	// Only sleep if we're not replaying a simulation
	// #BG TODO If this becomes an issue, recorded tracks should track awake state as well as transforms
	if (DynamicState == (uint8)EObjectStateTypeEnum::Chaos_Object_Sleeping)
	{
		Handle->SetObjectStateLowLevel(Chaos::EObjectStateType::Sleeping);
	}
	else if (DynamicState == (uint8)EObjectStateTypeEnum::Chaos_Object_Kinematic)
	{
		Handle->SetObjectStateLowLevel(Chaos::EObjectStateType::Kinematic);
	}
	else if (DynamicState == (uint8)EObjectStateTypeEnum::Chaos_Object_Static)
	{
		Handle->SetObjectStateLowLevel(Chaos::EObjectStateType::Static);
	}
	else
	{
		Handle->SetObjectStateLowLevel(Chaos::EObjectStateType::Dynamic);
	}
}

//==============================================================================
// FGeometryCollectionPhysicsProxy
//==============================================================================

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FGeometryCollectionPhysicsProxy::FGeometryCollectionPhysicsProxy(
	UObject* InOwner,
	FGeometryDynamicCollection& GameThreadCollectionIn,
	const FSimulationParameters& SimulationParameters,
	FCollisionFilterData InSimFilter,
	FCollisionFilterData InQueryFilter,
	FGuid InCollectorGuid,
	const Chaos::EMultiBufferMode BufferMode)
	: Base(InOwner)
	, Parameters(SimulationParameters)
	, WorldTransform_External(FTransform::Identity)
	, NumTransforms(INDEX_NONE)
	, NumEffectiveParticles(INDEX_NONE)
	, BaseParticleIndex(INDEX_NONE)
	, CollisionParticlesPerObjectFraction(CollisionParticlesPerObjectFractionDefault)
	, bIsGameThreadWorldTransformDirty(false)
	, bHasBuiltGeometryOnPT(false)
	, bHasBuiltGeometryOnGT(false)
	, IsObjectDynamic(false)
	, IsObjectLoading(true)
	, IsObjectDeleting(false)
	, bSkipChildToParentUpdateWhenInClusterUnion(false)
	, PhysicsThreadCollection(Parameters.RestCollectionShared)
	, GameThreadCollection(GameThreadCollectionIn)
#if WITH_EDITORONLY_DATA
	, CollectorGuid(InCollectorGuid)
#endif
{
	Chaos::Filter::FCombinedShapeFilterData CombinedShapeFilter = Chaos::Filter::FShapeFilterBuilder::BuildFromLegacyShapeFilter(InQueryFilter, InSimFilter);
	ShapeFilterData = CombinedShapeFilter.GetShapeFilterData();
	ShapeInstanceData = CombinedShapeFilter.GetInstanceData();

	// We rely on a guarded buffer.
	check(BufferMode == Chaos::EMultiBufferMode::TripleGuarded);
	InterpolationData = MakeUnique<FProxyInterpolationBase>();
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FGeometryCollectionPhysicsProxy::FGeometryCollectionPhysicsProxy(
	UObject* InOwner,
	TSharedRef<FGeometryDynamicCollection> GameThreadCollectionSharedRefIn,
	const FSimulationParameters& SimulationParameters,
	FCollisionFilterData InSimFilter,
	FCollisionFilterData InQueryFilter,
	FGuid InCollectorGuid,
	const Chaos::EMultiBufferMode BufferMode)
	: Base(InOwner)
	, Parameters(SimulationParameters)
	, WorldTransform_External(FTransform::Identity)
	, NumTransforms(INDEX_NONE)
	, NumEffectiveParticles(INDEX_NONE)
	, BaseParticleIndex(INDEX_NONE)
	, CollisionParticlesPerObjectFraction(CollisionParticlesPerObjectFractionDefault)
	, bIsGameThreadWorldTransformDirty(false)
	, bHasBuiltGeometryOnPT(false)
	, bHasBuiltGeometryOnGT(false)
	, IsObjectDynamic(false)
	, IsObjectLoading(true)
	, IsObjectDeleting(false)
	, bSkipChildToParentUpdateWhenInClusterUnion(false)
	, PhysicsThreadCollection(Parameters.RestCollectionShared)
	, GameThreadCollection(GameThreadCollectionSharedRefIn.Get())
	, GameThreadCollectionSharedPtr(GameThreadCollectionSharedRefIn.ToSharedPtr())
#if WITH_EDITORONLY_DATA
	, CollectorGuid(InCollectorGuid)
#endif
{
	Chaos::Filter::FCombinedShapeFilterData CombinedShapeFilter = Chaos::Filter::FShapeFilterBuilder::BuildFromLegacyShapeFilter(InQueryFilter, InSimFilter);
	ShapeFilterData = CombinedShapeFilter.GetShapeFilterData();
	ShapeInstanceData = CombinedShapeFilter.GetInstanceData();

	// We rely on a guarded buffer.
	check(BufferMode == Chaos::EMultiBufferMode::TripleGuarded);
	InterpolationData = MakeUnique<FProxyInterpolationBase>();
}

FGeometryCollectionPhysicsProxy::FGeometryCollectionPhysicsProxy(
	UObject* InOwner,
	TSharedRef<FGeometryDynamicCollection> GameThreadCollectionSharedRefIn,
	const FSimulationParameters& SimulationParameters,
	const Chaos::Filter::FShapeFilterData& InShapeFilterData,
	const Chaos::Filter::FInstanceData& InShapeInstanceData,
	FGuid InCollectorGuid,
	const Chaos::EMultiBufferMode BufferMode)
	: Base(InOwner)
	, Parameters(SimulationParameters)
	, WorldTransform_External(FTransform::Identity)
	, NumTransforms(INDEX_NONE)
	, NumEffectiveParticles(INDEX_NONE)
	, BaseParticleIndex(INDEX_NONE)
	, CollisionParticlesPerObjectFraction(CollisionParticlesPerObjectFractionDefault)
	, bIsGameThreadWorldTransformDirty(false)
	, bHasBuiltGeometryOnPT(false)
	, bHasBuiltGeometryOnGT(false)
	, IsObjectDynamic(false)
	, IsObjectLoading(true)
	, IsObjectDeleting(false)
	, bSkipChildToParentUpdateWhenInClusterUnion(false)
	, ShapeFilterData(InShapeFilterData)
	, ShapeInstanceData(InShapeInstanceData)
	, PhysicsThreadCollection(Parameters.RestCollectionShared)
	, GameThreadCollection(GameThreadCollectionSharedRefIn.Get())
	, GameThreadCollectionSharedPtr(GameThreadCollectionSharedRefIn.ToSharedPtr())
#if WITH_EDITORONLY_DATA
	, CollectorGuid(InCollectorGuid)
#endif
{
	// We rely on a guarded buffer.
	check(BufferMode == Chaos::EMultiBufferMode::TripleGuarded);
	InterpolationData = MakeUnique<FProxyInterpolationBase>();
}

FGeometryCollectionPhysicsProxy::~FGeometryCollectionPhysicsProxy()
{}

float ReportHighParticleFraction = -1.f;
FAutoConsoleVariableRef CVarReportHighParticleFraction(TEXT("p.gc.ReportHighParticleFraction"), ReportHighParticleFraction, TEXT("Report any objects with particle fraction above this threshold"));

bool bBuildGeometryForChildrenOnPT = true;
FAutoConsoleVariableRef CVarbBuildGeometryForChildrenOnPT(
	TEXT("p.gc.BuildGeometryForChildrenOnPT"), 
	bBuildGeometryForChildrenOnPT, 
	TEXT("DEPRECATED : use the 'Optimize For Memory' option from project settings instead - "
		 "If true build all children geometry on Physics Thread at initilaization time, otherwise wait until destruction occurs.")
);

bool bBuildGeometryForChildrenOnGT = true;
FAutoConsoleVariableRef CVarbBuildGeometryForChildrenOnGT(
	TEXT("p.gc.BuildGeometryForChildrenOnGT"), 
	bBuildGeometryForChildrenOnGT, 
	TEXT("DEPRECATED : use the 'Optimize For Memory' option from project settings instead - "
		 "If true build all children geometry  on Game Thread at initilaization time, otherwise wait until destruction occurs.")
);

bool bCreateGTParticleForChildren = true;
FAutoConsoleVariableRef CVarCreateGTParticleForChildren(
	TEXT("p.gc.CreateGTParticlesForChildren"), 
	bCreateGTParticleForChildren, 
	TEXT("DEPRECATED : use the 'Optimize For Memory' option from project settings instead - "
		 "If true create all children particles at initilaization time, otherwise wait until destruction occurs.")
);

bool bRemoveImplicitsInDynamicCollections = false;
FAutoConsoleVariableRef CVarbRemoveImplicitsInDynamicCollections(
	TEXT("p.gc.RemoveImplicitsInDynamicCollections"), 
	bRemoveImplicitsInDynamicCollections, 
	TEXT("DEPRECATED : use the 'Optimize For Memory' option from project settings instead - "
		 "This cvar has an impact only if geometry are not added for children. It removes implicits from the Dynamic Collections, and recreate then from the rest collection. "
	  	 "Using this cvar could have an impact if geometry are updated from the dynamic collection on the GT, then those changes won't be ported to the PT.")
);

namespace GeometryCollectionPhysicsProxy::Private
{
	CHAOS_API bool BuildGeometryForChildrenOnPT(const FSimulationParameters& Params)
	{
		return bBuildGeometryForChildrenOnPT && !Params.bOptimizeForRuntimeMemory;
	}

	bool BuildGeometryForChildrenOnGT(const FSimulationParameters& Params)
	{
		return bBuildGeometryForChildrenOnGT && !Params.bOptimizeForRuntimeMemory;
	}

	bool CreateGTParticleForChildren(const FSimulationParameters& Params)
	{
		return bCreateGTParticleForChildren && !Params.bOptimizeForRuntimeMemory;
	}

	bool RemoveImplicitsInDynamicCollections(const FSimulationParameters& Params)
	{
		return bRemoveImplicitsInDynamicCollections || Params.bOptimizeForRuntimeMemory;
	}
}

void FGeometryCollectionPhysicsProxy::Initialize(Chaos::FPBDRigidsEvolutionBase *Evolution)
{
	using namespace GeometryCollectionPhysicsProxy::Private;

	check(IsInGameThread());
	//
	// Game thread initialization. 
	//
	//  1) Create a input buffer to store all game thread side data. 
	//  2) Populate the buffer with the necessary data.
	//  3) Deep copy the data to the other buffers. 
	//
	FGeometryDynamicCollection& DynamicCollection = GameThreadCollection;

	InitializeDynamicCollection(DynamicCollection, *Parameters.RestCollectionShared, Parameters);

	NumTransforms = DynamicCollection.NumElements(FGeometryCollection::TransformGroup);
	BaseParticleIndex = 0; // Are we always zero indexed now?
	
	NumEffectiveParticles = CalculateEffectiveParticles(DynamicCollection, NumTransforms, Parameters.MaxSimulatedLevel, Parameters.EnableClustering, GetOwner(), EffectiveParticles);

	ParticlesToDirty.Init(false, NumEffectiveParticles);
	SolverClusterID.Init(nullptr, NumEffectiveParticles);
	SolverClusterHandles.Init(nullptr, NumEffectiveParticles);
	SolverParticleHandles.Init(nullptr, NumEffectiveParticles);
	GTParticles.SetNum(NumEffectiveParticles);
	UniqueIdxs.SetNum(NumEffectiveParticles);
	
	FromParticleToTransformIndex.SetNum(NumEffectiveParticles);
	FromTransformToParticleIndex.SetNum(NumTransforms);
	PhysicsObjects.Empty();
	PhysicsObjects.Reserve(NumEffectiveParticles);
	int32 RedirectIndex = 0;
	for (int32 Index = 0; Index < NumTransforms; ++Index)
	{
		if (EffectiveParticles[Index])
		{
			FromTransformToParticleIndex[Index] = RedirectIndex;
			FromParticleToTransformIndex[RedirectIndex] = Index;
			PhysicsObjects.Emplace(Chaos::FPhysicsObjectFactory::CreatePhysicsObject(this, Index, FName(Parameters.RestCollectionShared->BoneName[Index])));
			RedirectIndex++;
		}
		else
		{
			FromTransformToParticleIndex[Index] = INDEX_NONE;
		}
	}

	check(RedirectIndex == NumEffectiveParticles);

	// we need to make sure the world transform is kept up to date on the game thread 
	WorldTransform_External = Parameters.WorldTransform;
	PreviousWorldTransform_External = WorldTransform_External;

	//
	// Collision vertices down sampling validation.  
	//
	CollisionParticlesPerObjectFraction = Parameters.CollisionSampleFraction * CollisionParticlesPerObjectFractionDefault;
	if (ReportHighParticleFraction > 0)
	{
		for (const FSharedSimulationSizeSpecificData& Data : Parameters.Shared.SizeSpecificData)
		{
			if (ensure(Data.CollisionShapesData.Num()))
			{
				if (Data.CollisionShapesData[0].CollisionParticleData.CollisionParticlesFraction >= ReportHighParticleFraction)
				{
					ensureMsgf(false, TEXT("Collection with small particle fraction"));
					UE_LOGF(LogChaos, Warning, "Collection with small particle fraction(%f):%ls", Data.CollisionShapesData[0].CollisionParticleData.CollisionParticlesFraction, *Parameters.Name);
				}
			}
		}
	}

	// Initialise GT/External particles
	check(NumTransforms == DynamicCollection.GetNumTransforms());
	// make sure we copy the anchored information over to the physics thread collection
	const Chaos::Facades::FCollectionAnchoringFacade DynamicCollectionAnchoringFacade(DynamicCollection);
	Chaos::Facades::FCollectionAnchoringFacade PhysicsThreadCollectionAnchoringFacade(PhysicsThreadCollection);
	PhysicsThreadCollectionAnchoringFacade.CopyAnchoredAttribute(DynamicCollectionAnchoringFacade);

	const FVector Scale = Parameters.WorldTransform.GetScale3D();
	const TManagedArray<float>& Mass = Parameters.RestCollectionShared->GetAttribute<float>(MassAttributeName, FTransformCollection::TransformGroup);

	TManagedArray<Chaos::FImplicitObjectPtr>& Implicits = GameThreadCollection.ModifyAttribute<Chaos::FImplicitObjectPtr>(FGeometryDynamicCollection::ImplicitsAttribute, FTransformCollection::TransformGroup);
	if (ensure(NumTransforms == Implicits.Num() && NumEffectiveParticles == GTParticles.Num())) // Implicits are in the transform group so this invariant should always hold
	{
		constexpr bool bInitializationTime = true;
		bHasBuiltGeometryOnGT = BuildGeometryForChildrenOnGT(Parameters);
		CreateGTParticles(Implicits, Evolution, bInitializationTime);

		// Skip simplicials, as they're owned by unique pointers.
		static const FAttributeAndGroupId SkipList[] =
		{
			{ FGeometryDynamicCollection::SimplicialsAttribute, FTransformCollection::TransformGroup },
		};
		static constexpr int32 SkipListSize = sizeof(SkipList) / sizeof(FAttributeAndGroupId);

		// Adding the Implicits to PhysicsThreadCollection, before it copies all matching attributes
		PhysicsThreadCollection.AddAttribute<Chaos::FImplicitObjectPtr>(FGeometryDynamicCollection::ImplicitsAttribute, FTransformCollection::TransformGroup);
		PhysicsThreadCollection.CopyMatchingAttributesFrom(DynamicCollection, MakeArrayView(SkipList, SkipListSize));
		PhysicsThreadCollection.CopyInitialVelocityAttributesFrom(DynamicCollection);

		if (Parameters.EnableClustering)
		{
			// make sure we set Activate the right way when clustering is enabled ( only root should be enabled at start ) 
			for (int32 ParticleIndex = 0; ParticleIndex < NumEffectiveParticles; ++ParticleIndex)
			{
				const int32 TransformIndex = FromParticleToTransformIndex[ParticleIndex];
				const bool bIsRoot = !GameThreadCollection.GetHasParent(TransformIndex);
				GameThreadCollection.Active[TransformIndex] = bIsRoot;
				PhysicsThreadCollection.Active[TransformIndex] = bIsRoot;

				if (FParticle* P = GTParticles[ParticleIndex].Get())
				{
					if (P != nullptr)
					{
						P->SetDisabled(!bIsRoot);
					}
				}
			}
		}
	}

	if (bHasBuiltGeometryOnGT || RemoveImplicitsInDynamicCollections(Parameters))
	{
		// The Implicits attributes from the Dynamic Collection are just used for initialization, after they can be removed and so free some memory.
		GameThreadCollection.RemoveAttribute(FGeometryDynamicCollection::ImplicitsAttribute, FTransformCollection::TransformGroup);
	}
	// If GT particle have been created, we can use directly the UniqueIdx from the GTParticle, then free up some space.
	if ((BuildGeometryForChildrenOnGT(Parameters) || CreateGTParticleForChildren(Parameters)) && bEmptyUniqueIndicesOnGT)
	{
		UniqueIdxs.Empty();
	}
}

void FGeometryCollectionPhysicsProxy::CreateGTParticles(TManagedArray<Chaos::FImplicitObjectPtr>& Implicits, Chaos::FPBDRigidsEvolutionBase* Evolution, bool bInitializationTime)
{
	using namespace GeometryCollectionPhysicsProxy::Private;

	const Chaos::Facades::FCollectionAnchoringFacade DynamicCollectionAnchoringFacade(GameThreadCollection);
	const FVector Scale = Parameters.WorldTransform.GetScale3D();
	const TManagedArray<float>& Mass = Parameters.RestCollectionShared->GetAttribute<float>(MassAttributeName, FTransformCollection::TransformGroup);
	const TManagedArray<FTransform>& MassToLocal = Parameters.RestCollectionShared->GetAttribute<FTransform>(MassToLocalAttributeName, FTransformCollection::TransformGroup);
	const TManagedArray<int32>& Level = Parameters.RestCollectionShared->GetAttribute<int32>(LevelAttributeName, FTransformCollection::TransformGroup);

	TArray<int32> ChildrenToCheckForParentFix;
	if (!bInitializationTime && !CreateGTParticleForChildren(Parameters) && !BuildGeometryForChildrenOnGT(Parameters))
	{
		GTParticleToIndices.Reserve(NumEffectiveParticles);
	}

	for (int32 ParticleIndex = 0; ParticleIndex < NumEffectiveParticles; ++ParticleIndex)
	{
		const int32 TransformIndex = FromParticleToTransformIndex[ParticleIndex];

		FParticle* P = GTParticles[ParticleIndex].Get();
		// Generate all particles unique idx at initialization time
		if (bInitializationTime)
		{
			UniqueIdxs[ParticleIndex] = Evolution->GenerateUniqueIdx();
		}

		if ((bInitializationTime && TransformIndex == Parameters.InitialRootIndex) || // When initializing always create particle for the root
			((bInitializationTime && (CreateGTParticleForChildren(Parameters) || BuildGeometryForChildrenOnGT(Parameters))) || // When initializing create all particles if one of the flag is true 
				(!bInitializationTime && TransformIndex != Parameters.InitialRootIndex && !CreateGTParticleForChildren(Parameters) && !BuildGeometryForChildrenOnGT(Parameters)))) // When not initializing create other particles if flag was set to not create
		{
			GTParticles[ParticleIndex] = FParticle::CreateParticle();
			P = GTParticles[ParticleIndex].Get();
			GTParticleToIndices.FindOrAdd(P).TransformGroupIndex = TransformIndex;
			GTParticles[ParticleIndex]->SetUniqueIdx(UniqueIdxs[ParticleIndex]);
#if CHAOS_DEBUG_NAME
				P->SetDebugName(MakeShared<FString, ESPMode::ThreadSafe>(FString::Printf(TEXT("%s-%d"), *Parameters.Name, TransformIndex)));
#endif
			const float ScaledMass = AdjustMassForScale(Mass[TransformIndex]);

			// Note that this transform must match the physics thread transform computation for initialization.
			// Take for example, a geometry collection in the editor that is linked with a joint constraint. The joint
			// constraint will query the game thread particle position/rotation for the geometry collection to compute its
			// reference frame. If that position/rotation does not match up with the physics thread's position/rotation,
			// the geometry collection particle will have an added velocity computed by the joint constraint solver.
			const FTransform& T = MassToLocal[TransformIndex] * FTransform(GameThreadCollection.GetTransform(TransformIndex)) * Parameters.WorldTransform;
			P->SetX(T.GetTranslation(), false);
			P->SetR(T.GetRotation(), false);
			P->SetM(ScaledMass);
			P->SetUserData(Parameters.UserData);
			P->SetProxy(this);
		}
		if (bInitializationTime && TransformIndex == Parameters.InitialRootIndex && Parameters.bUseStaticMeshCollisionForTraces && CreateTraceCollisionGeometryCallback != nullptr)
		{
			const FTransform ToLocal = MassToLocal[TransformIndex].Inverse();
			TArray<Chaos::FImplicitObjectPtr> Geoms;
			Chaos::FShapesArray Shapes;
			CreateTraceCollisionGeometryCallback(ToLocal, Geoms, Shapes);

			Chaos::FImplicitObjectPtr ImplicitGeometry = MakeImplicitObjectPtr<Chaos::FImplicitObjectUnion>(MoveTemp(Geoms));
			P->SetGeometry(ImplicitGeometry);
		}
		else if (bInitializationTime == (TransformIndex == Parameters.InitialRootIndex) || BuildGeometryForChildrenOnGT(Parameters))
		{
			Chaos::FImplicitObjectPtr ImplicitGeometry = Implicits[TransformIndex];
			if (ImplicitGeometry && !Scale.Equals(FVector::OneVector))
			{
				ImplicitGeometry = ImplicitGeometry->CopyGeometryWithScale(Scale);
			}
			P->SetGeometry(ImplicitGeometry);
		}
		// Reset the root in the ManagedArray if not in mode bInitializeRootOnly
		if (!bInitializationTime && TransformIndex == Parameters.InitialRootIndex)
		{
			Implicits[TransformIndex] = P->GetGeometry();
		}

		if (bInitializationTime == (TransformIndex == Parameters.InitialRootIndex) || BuildGeometryForChildrenOnGT(Parameters))
		{
			if (DynamicCollectionAnchoringFacade.IsAnchored(TransformIndex))
			{
				P->SetObjectState(Chaos::EObjectStateType::Kinematic, false, false);
			}

			// IMPORTANT: we need to set the right spatial index because GT particle is static and PT particle is rigid
			// this is causing a mismatch when using the separate acceleration structures optimization which can cause crashes when destroying the particle while async tracing 
			// todo(chaos) we should eventually refactor this code to use rigid particles on the GT side for geometry collection  
			P->SetSpatialIdx(Chaos::FSpatialAccelerationIdx{ 0,1 });

			if (Chaos::AccelerationStructureSplitStaticAndDynamic == 1)
			{
				P->SetSpatialIdx(Chaos::FSpatialAccelerationIdx{ 0,1 });
			}
			else
			{
				P->SetSpatialIdx(Chaos::FSpatialAccelerationIdx{ 0,0 });
			}

			const bool bIsOneWayInteraction = (Parameters.OneWayInteractionLevel >= 0) && (Level[TransformIndex] >= Parameters.OneWayInteractionLevel);
			P->SetOneWayInteraction(bIsOneWayInteraction);
		}
		// this step is necessary for Phase 2 where we need to walk back the hierarchy from children to parent 
		if (bGeometryCollectionAlwaysGenerateGTCollisionForClusters && !GameThreadCollection.HasChildren(TransformIndex))
		{
			ChildrenToCheckForParentFix.Add(TransformIndex);
		}
	}

	if (bGeometryCollectionAlwaysGenerateGTCollisionForClusters)
	{
		// second phase: fixing parent geometries
		// @todo(chaos) this could certainly be done ahead at generation time rather than runtime
		TSet<int32> ParentToPotentiallyFix;
		while (ChildrenToCheckForParentFix.Num())
		{
			// step 1 : find parents
			for (const int32 ChildIndex : ChildrenToCheckForParentFix)
			{
				const int32 ParentIndex = GameThreadCollection.GetParent(ChildIndex);
				if (ParentIndex != INDEX_NONE)
				{
					ParentToPotentiallyFix.Add(ParentIndex);
				}
			}

			// step 2: fix the parent if necessary
			for (const int32 ParentToFixIndex : ParentToPotentiallyFix)
			{
				if (Implicits[ParentToFixIndex] == nullptr)
				{
					const Chaos::FRigidTransform3 ParentShapeTransform = MassToLocal[ParentToFixIndex];

					// let's make sure all our children have an implicit defined, other wise, postpone to next iteration 
					bool bAllChildrenHaveCollision = true;

					GameThreadCollection.IterateThroughChildren(ParentToFixIndex, [&](int32 ChildIndex)
						{
							// defer if any of the children is a cluster with no collision yet generated 
							if (Implicits[ChildIndex] == nullptr && GameThreadCollection.HasChildren(ChildIndex))
							{
								bAllChildrenHaveCollision = false;
								return false;
							}
							return true;
						});

					if (bAllChildrenHaveCollision)
					{
						// Make a union of the children geometry
						TArray<Chaos::FImplicitObjectPtr> ChildImplicits;
						GameThreadCollection.IterateThroughChildren(ParentToFixIndex, [&](int32 ChildIndex)
							{
								const Chaos::FImplicitObjectPtr& ChildImplicit = Implicits[ChildIndex];
								if (ChildImplicit)
								{
									const Chaos::FRigidTransform3 ChildShapeTransform = MassToLocal[ChildIndex] * FTransform(GameThreadCollection.GetTransform(ChildIndex));
									const Chaos::FRigidTransform3 RelativeShapeTransform = ChildShapeTransform.GetRelativeTransform(ParentShapeTransform);

									Chaos::FImplicitObjectPtr TransformedChildImplicit = MakeTransformImplicitObject(*ChildImplicit, RelativeShapeTransform);

									// if this remains a union we need to unpack it 
									if (TransformedChildImplicit->IsUnderlyingUnion())
									{
										// we just move the array for children in the new array 
										Chaos::FImplicitObjectUnion& Union = static_cast<Chaos::FImplicitObjectUnion&>(*TransformedChildImplicit);
										ChildImplicits.Append(MoveTemp(Union.GetObjects()));
									}
									else
									{
										ChildImplicits.Add(MoveTemp(TransformedChildImplicit));
									}
								}
								return true;
							});
						if (ChildImplicits.Num() > 0)
						{
							Chaos::FImplicitObject* UnionImplicit = new Chaos::FImplicitObjectUnion(MoveTemp(ChildImplicits));
							Implicits[ParentToFixIndex] = Chaos::FImplicitObjectPtr(UnionImplicit);
						}
						if (FromTransformToParticleIndex[ParentToFixIndex] != INDEX_NONE && GTParticles[FromTransformToParticleIndex[ParentToFixIndex]] != nullptr)
						{
							GTParticles[FromTransformToParticleIndex[ParentToFixIndex]]->SetGeometry(Implicits[ParentToFixIndex]);
						}
					}
				}
			}

			// step 3 : make the parent the new child to go up the hierarchy and continue the fixing
			ChildrenToCheckForParentFix = ParentToPotentiallyFix.Array();
			ParentToPotentiallyFix.Reset();
		}
	}

	// Phase 3 : finalization of shapes
	for (int32 Index = 0; Index < NumEffectiveParticles; ++Index)
	{
		FParticle* P = GTParticles[Index].Get();
		if (P != nullptr)
		{
			const Chaos::FShapesArray& Shapes = P->ShapesArray();
			const int32 NumShapes = Shapes.Num();
			for (int32 ShapeIndex = 0; ShapeIndex < NumShapes; ++ShapeIndex)
			{
				Chaos::FPerShapeData* Shape = Shapes[ShapeIndex].Get();
				Shape->SetShapeFilterData(ShapeFilterData);
				Shape->SetFilterInstanceData(ShapeInstanceData);
				Shape->SetProxy(this);
				Shape->SetMaterial(Parameters.PhysicalMaterialHandle);
			}
		}
	}
}

void FGeometryCollectionPhysicsProxy::CreateChildrenGeometry_External()
{
	using namespace GeometryCollectionPhysicsProxy::Private;

	if (!bHasBuiltGeometryOnGT)
	{
		if (Chaos::FPhysicsSolver* RBDSolver = GetSolver<Chaos::FPhysicsSolver>())
		{
			if (RemoveImplicitsInDynamicCollections(Parameters))
			{
				GameThreadCollection.AddAttribute<Chaos::FImplicitObjectPtr>(FGeometryDynamicCollection::ImplicitsAttribute, FTransformCollection::TransformGroup);
				GameThreadCollection.CopyAttribute(*Parameters.RestCollectionShared, FGeometryDynamicCollection::ImplicitsAttribute, FTransformCollection::TransformGroup);
			}

			TManagedArray<Chaos::FImplicitObjectPtr>& Implicits = GameThreadCollection.ModifyAttribute<Chaos::FImplicitObjectPtr>(FGeometryDynamicCollection::ImplicitsAttribute, FTransformCollection::TransformGroup);
			CreateGTParticles(Implicits, RBDSolver->GetEvolution(), /*bInitializationTime*/false);
			SyncParticles_External();

			if (bEmptyUniqueIndicesOnGT)
			{
				UniqueIdxs.Empty();
			}

			// The Implicits attributes from the Dynamic Collection are just used for initialization, after they can be removed and so free some memory.
			GameThreadCollection.RemoveAttribute(FGeometryDynamicCollection::ImplicitsAttribute, FTransformCollection::TransformGroup);
			bHasBuiltGeometryOnGT = true;

			if (PostParticlesCreatedCallback)
			{
				PostParticlesCreatedCallback();
			}
		}
	}
}


void FGeometryCollectionPhysicsProxy::InitializeDynamicCollection(FGeometryDynamicCollection& DynamicCollection, const FGeometryCollection& RestCollection, const FSimulationParameters& Params)
{
	// @todo(GCCollisionShapes) : add support for multiple shapes, currently just one. 

	// 
	// This function will use the rest collection to populate the dynamic collection. 
	//

	static const FAttributeAndGroupId SkipList[] =
	{
		{ FTransformCollection::TransformAttribute, FTransformCollection::TransformGroup },
		{ FTransformCollection::ParentAttribute, FTransformCollection::TransformGroup },
		{ FTransformCollection::ChildrenAttribute, FTransformCollection::TransformGroup },
		{ FGeometryCollection::SimulationTypeAttribute, FTransformCollection::TransformGroup },
		{ FGeometryDynamicCollection::SimplicialsAttribute, FTransformCollection::TransformGroup },
		{ FGeometryDynamicCollection::ActiveAttribute, FTransformCollection::TransformGroup },
	};
	static const int32 SkipListSize = sizeof(SkipList) / sizeof(FAttributeAndGroupId);

	DynamicCollection.AddAttribute<Chaos::FImplicitObjectPtr>(FGeometryDynamicCollection::ImplicitsAttribute, FTransformCollection::TransformGroup);
	DynamicCollection.CopyMatchingAttributesFrom(RestCollection, MakeArrayView(SkipList, SkipListSize));

	// User defined initial velocities need to be populated. 
	if (Params.InitialVelocityType == EInitialVelocityTypeEnum::Chaos_Initial_Velocity_User_Defined)
	{
		FGeometryDynamicCollection::FInitialVelocityFacade InitialVelocityFacade = DynamicCollection.GetInitialVelocityFacade();
		InitialVelocityFacade.DefineSchema();
		InitialVelocityFacade.Fill(FVector3f(Params.InitialLinearVelocity), FVector3f(Params.InitialAngularVelocity));
	}

	// Process Activity
	{
		const int32 NumTransforms = DynamicCollection.SimulatableParticles.Num();
		if (!RestCollection.HasAttribute(FGeometryCollection::SimulatableParticlesAttribute, FTransformCollection::TransformGroup))
		{
			// If no simulation data is available then default to the simulation of just the rigid geometry.
			for (int32 TransformIdx = 0; TransformIdx < NumTransforms; TransformIdx++)
			{
				if (DynamicCollection.HasChildren(TransformIdx))
				{
					DynamicCollection.SimulatableParticles[TransformIdx] = false;
				}
				else
				{
					DynamicCollection.SimulatableParticles[TransformIdx] = DynamicCollection.Active[TransformIdx];
				}
			}
		}
	}
}

int32 ReportTooManyChildrenNum = -1;
FAutoConsoleVariableRef CVarReportTooManyChildrenNum(TEXT("p.ReportTooManyChildrenNum"), ReportTooManyChildrenNum, TEXT("Issue warning if more than this many children exist in a single cluster"));

void FGeometryCollectionPhysicsProxy::CreateNonClusteredParticles(Chaos::FPBDRigidsSolver* RigidsSolver, const FGeometryCollection& RestCollection, const FGeometryDynamicCollection& DynamicCollection)
{
	LLM_SCOPE_BYNAME(TEXT("Physics/NonClusteredParticles"));
	const TManagedArray<bool>& SimulatableParticles = DynamicCollection.SimulatableParticles;
	
	//const int NumRigids = 0; // ryan - Since we're doing SOA, we start at zero?
	int NumRigids = 0;
	BaseParticleIndex = NumRigids;

	// Gather unique indices from GT to pass into PT handle creation
	TArray<Chaos::FUniqueIdx> UniqueIndices;
	UniqueIndices.Reserve(SimulatableParticles.Num());

	// Count geometry collection leaf node particles to add
	int NumSimulatedParticles = 0;
	for (int32 ParticleIndex = 0; ParticleIndex < NumEffectiveParticles; ++ParticleIndex)
	{
		const int32 TransformIndex = FromParticleToTransformIndex[ParticleIndex];
		NumSimulatedParticles += SimulatableParticles[TransformIndex];
		if (SimulatableParticles[TransformIndex] && !RestCollection.IsClustered(TransformIndex) && RestCollection.IsGeometry(TransformIndex))
		{
			NumRigids++;
			Chaos::FUniqueIdx ExistingIndex;
			if (GTParticles[ParticleIndex] == nullptr)
			{
				ExistingIndex = UniqueIdxs[ParticleIndex];
			}
			else
			{
				ExistingIndex = GTParticles[ParticleIndex]->UniqueIdx();
			}
			UniqueIndices.Add(ExistingIndex);
		}
	}

	// Add entries into simulation array
	RigidsSolver->GetEvolution()->ReserveParticles(NumSimulatedParticles);
	TArray<Chaos::TPBDGeometryCollectionParticleHandle<Chaos::FReal, 3>*> Handles = RigidsSolver->GetEvolution()->CreateGeometryCollectionParticles(NumRigids, UniqueIndices.GetData());

	int32 NextIdx = 0;
	for (int32 ParticleIndex = 0; ParticleIndex < NumEffectiveParticles; ++ParticleIndex)
	{
		const int32 TransformIndex = FromParticleToTransformIndex[ParticleIndex];
		SolverParticleHandles[ParticleIndex] = nullptr;
		if (EffectiveParticles[TransformIndex] && SimulatableParticles[TransformIndex] && !RestCollection.IsClustered(TransformIndex))
		{
			// todo: Unblocked read access of game thread data on the physics thread.

			Chaos::TPBDGeometryCollectionParticleHandle<Chaos::FReal, 3>* Handle = Handles[NextIdx++];

			Handle->SetPhysicsProxy(this);

			SolverParticleHandles[ParticleIndex] = Handle;
			HandleToTransformGroupIndex.Add(Handle, TransformIndex);

			// We're on the physics thread here but we've already set up the GT particles and we're just linking here
			Handle->GTGeometryParticle() = GTParticles[ParticleIndex].Get();

			check(SolverParticleHandles[ParticleIndex]->GetParticleType() == Handle->GetParticleType());
			RigidsSolver->GetEvolution()->RegisterParticle(Handle);
		}
	}

	// anchors
	
}

Chaos::FPBDRigidClusteredParticleHandle* FGeometryCollectionPhysicsProxy::FindClusteredParticleHandleByItemIndex_Internal(FGeometryCollectionItemIndex ItemIndex) const
{
	Chaos::FPBDRigidClusteredParticleHandle* ResultHandle = nullptr;;
	if (ItemIndex.IsInternalCluster())
	{
		const int32 InternalClusterUniqueIdx = ItemIndex.GetInternalClusterIndex();
		if (Chaos::FPBDRigidClusteredParticleHandle* const* InternalClusterHandle = UniqueIdxToInternalClusterHandle.Find(InternalClusterUniqueIdx))
		{
			ResultHandle = *InternalClusterHandle;
		}
	}
	else
	{
		const int32 TransformIndex = ItemIndex.GetTransformIndex();
		if (FromTransformToParticleIndex.IsValidIndex(TransformIndex))
		{
			const int32 ParticleIndex = FromTransformToParticleIndex[TransformIndex];
			if (SolverParticleHandles.IsValidIndex(ParticleIndex))
			{
				ResultHandle = SolverParticleHandles[ParticleIndex];
			}
		}
	}
	return ResultHandle;
}

void FGeometryCollectionPhysicsProxy::SetSleepingState(const Chaos::FPBDRigidsSolver& RigidsSolver)
{
	for (FClusterHandle* Handle: SolverParticleHandles)
	{
		if (Handle && !Handle->Disabled() && Handle->IsDynamic())
		{
			// Sleeping Geometry Collections:
			//   A sleeping geometry collection is dynamic internally, and then the top level
			//   active clusters are set to sleeping. Sleeping is not propagated up from the 
			//   leaf nodes like kinematic or dynamic clusters.
			RigidsSolver.GetEvolution()->SetParticleObjectState(Handle, Chaos::EObjectStateType::Sleeping);
		}
	}
}

void FGeometryCollectionPhysicsProxy::DirtyAllParticles(const Chaos::FPBDRigidsSolver& RigidsSolver)
{
	for (FClusterHandle* Handle: SolverParticleHandles)
	{
		if (Handle)
		{
			RigidsSolver.GetEvolution()->DirtyParticle(*Handle);
		}
	}
}

void FGeometryCollectionPhysicsProxy::UpdateDamageThreshold_Internal()
{
	ensure(SolverParticleHandles.Num() == SolverClusterHandles.Num());

	if (Chaos::FPhysicsSolver* RBDSolver = GetSolver<Chaos::FPhysicsSolver>())
	{
		Chaos::FRigidClustering& Clustering = RBDSolver->GetEvolution()->GetRigidClustering();

		const float StrainDefault = Parameters.DamageThreshold.Num() ? Parameters.DamageThreshold[0] : 0;

		for (int32 ParticleIndex = 0; ParticleIndex < NumEffectiveParticles; ParticleIndex++)
		{
			if (Chaos::FPBDRigidClusteredParticleHandle* Handle = SolverParticleHandles[ParticleIndex])
			{
				const int32 TransformIndex = FromParticleToTransformIndex[ParticleIndex];
				float DamageThreshold = StrainDefault;
				switch (Parameters.DamageModel)
				{
				case EDamageModelTypeEnum::Chaos_Damage_Model_UserDefined_Damage_Threshold:
					if (!Parameters.bUsePerClusterOnlyDamageThreshold)
					{
						const bool bIsACluster = (Handle->ClusterIds().NumChildren > 0);
						if (Parameters.EnableClustering || bIsACluster)
						{
							DamageThreshold = ComputeUserDefinedDamageThreshold_Internal(TransformIndex);
						}
					}
					break;
				case EDamageModelTypeEnum::Chaos_Damage_Model_Material_Strength_And_Connectivity_DamageThreshold:
					DamageThreshold = ComputeMaterialBasedDamageThreshold_Internal(TransformIndex);
					break;
				}
				Clustering.SetInternalStrain(Handle, DamageThreshold);
			}
		}

		// user defined legacy mode: propagate to the children form the cluster values
		if (Parameters.bUsePerClusterOnlyDamageThreshold && 
			Parameters.DamageModel == EDamageModelTypeEnum::Chaos_Damage_Model_UserDefined_Damage_Threshold)
		{
			for (int32 ParticleIndex = 0; ParticleIndex < NumEffectiveParticles; ParticleIndex++)
			{
				if (Chaos::FPBDRigidClusteredParticleHandle* Handle = SolverParticleHandles[ParticleIndex])
				{
					if (Chaos::FPBDRigidClusteredParticleHandle* ParentHandle = SolverClusterHandles[ParticleIndex])
					{
						const float DamageThreshold = ParentHandle->GetInternalStrains();
						Clustering.SetInternalStrain(Handle, DamageThreshold);
					}
				}
			}
		}
	}
}

void FGeometryCollectionPhysicsProxy::InitializeBodiesPT(Chaos::FPBDRigidsSolver* RigidsSolver, typename Chaos::FPBDRigidsSolver::FParticlesType& Particles)
{
	using namespace GeometryCollectionPhysicsProxy::Private;

	const FGeometryCollection* RestCollection = Parameters.RestCollectionShared.Get();
	const FGeometryDynamicCollection& DynamicCollection = PhysicsThreadCollection;

	bIsInitializedOnPhysicsThread = true;
	// Building geometry according to the cVar
	bHasBuiltGeometryOnPT = BuildGeometryForChildrenOnPT(Parameters);

	if (Parameters.Simulating && RestCollection)
	{
		const TManagedArray<int32>& BoneMap = RestCollection->BoneMap;
		const TManagedArray<int32>& SimulationType = RestCollection->SimulationType;
		const TManagedArray<FVector3f>& Vertex = RestCollection->Vertex;
		const TManagedArray<float>& Mass = RestCollection->GetAttribute<float>(MassAttributeName, FTransformCollection::TransformGroup);
		const TManagedArray<FVector3f>& InertiaTensor = RestCollection->GetAttribute<FVector3f>(InertiaTensorAttributeName, FTransformCollection::TransformGroup);

		check(NumTransforms == DynamicCollection.NumElements(FTransformCollection::TransformGroup));
		const TManagedArray<uint8>& DynamicState = DynamicCollection.DynamicState;
		const TManagedArray<bool>& SimulatableParticles = DynamicCollection.SimulatableParticles;
		const TManagedArray<FTransform>& MassToLocal = RestCollection->GetAttribute<FTransform>(MassToLocalAttributeName, FTransformCollection::TransformGroup);
		const TManagedArray<Chaos::FImplicitObjectPtr>* Implicits = DynamicCollection.FindAttribute<Chaos::FImplicitObjectPtr>(FGeometryDynamicCollection::ImplicitsAttribute, FTransformCollection::TransformGroup);
		if (!ensure(Implicits))
		{
			UE_LOGF(LogChaos, Error, "Geometry collection %ls - InitializeBodiesPT called with collision missing in the collection ", Owner ? *Owner->GetFullName(): TEXT("Unknown"));
		}

		using FSimplicialUniquePtr = TUniquePtr<FCollisionStructureManager::FSimplicial>;
		const TManagedArray<FSimplicialUniquePtr>* Simplicials = nullptr;
		if (Parameters.bUseSimplicialsWhenAvailable)
		{
			Simplicials = RestCollection->FindAttribute<FSimplicialUniquePtr>(DynamicCollection.SimplicialsAttribute, FTransformCollection::TransformGroup);
		}

		// In PushToPhysicsState, we're going to compute a relative transform from Parameters.PrevWorldTransform
		// to a particle's current world transform to get its relative transform. Then, that relative transform is
		// applied on top of the new Parameters.WorldTransform to get the final world transform. This is problematic
		// if the particle is initialized with some Parameters.WorldTransform because then the particle's world transform
		// will have Parameters.WorldTransform baked in but Parameters.PrevWorldTransform will be an identity. Then
		// when we go to set the kinematic target we will then effectively be applying Parameters.WorldTransform twice.
		Parameters.PrevWorldTransform = Parameters.WorldTransform;

		TArray<FTransform> Transform;
		GeometryCollectionAlgo::Private::GlobalMatrices(DynamicCollection, Transform);

		CreateNonClusteredParticles(RigidsSolver, *RestCollection, DynamicCollection);

		const float StrainDefault = Parameters.DamageThreshold.Num() ? Parameters.DamageThreshold[0] : 0;
		// Add the rigid bodies

		const Chaos::Facades::FCollectionAnchoringFacade AnchoringFacade(PhysicsThreadCollection);

		TArray<float> Masses;
		TArray<Chaos::FVec3f> Inertias;
		TArray < FTransform> Transforms;

		for (int32 Index = 0; Index < NumTransforms; Index++)
		{
			const float ScaledMass = AdjustMassForScale(Mass[Index]);
			const Chaos::FVec3f ScaledInertia = AdjustInertiaForScale((Chaos::FVec3f)InertiaTensor[Index]);
			Masses.Add(ScaledMass);
			Inertias.Add(ScaledInertia);
			const FTransform WorldTransform = MassToLocal[Index] * Transform[Index] * Parameters.WorldTransform;
			Transforms.Add(WorldTransform);
		}

		// Iterating over the geometry group is a fast way of skipping everything that's
		// not a leaf node, as each geometry has a transform index, which is a shortcut
		// for the case when there's a 1-to-1 mapping between transforms and geometries.
		// At the point that we start supporting instancing, this assumption will no longer
		// hold, and those reverse mappints will be INDEX_NONE.
		ParallelFor(NumEffectiveParticles, [&](int32 ParticleIndex)
		{
			const int32 TransformGroupIndex = FromParticleToTransformIndex[ParticleIndex];
			if (FClusterHandle* Handle = SolverParticleHandles[ParticleIndex])
			{
				const bool bIsAnchored = AnchoringFacade.IsAnchored(TransformGroupIndex);

				// Mass space -> Composed parent space -> world
				const Chaos::FReal ScaledMass = AdjustMassForScale(Mass[TransformGroupIndex]);
				const Chaos::FVec3f ScaledInertia = AdjustInertiaForScale((Chaos::FVec3f)InertiaTensor[TransformGroupIndex]);

				const bool bUseImplicit = (Implicits != nullptr) && (BuildGeometryForChildrenOnPT(Parameters) || (TransformGroupIndex == Parameters.InitialRootIndex));

				PopulateSimulatedParticle(
					Handle,
					Parameters.Shared,
					Simplicials? (*Simplicials)[TransformGroupIndex].Get(): nullptr,
					bUseImplicit? (*Implicits)[TransformGroupIndex] : nullptr,
					ShapeFilterData,
					ShapeInstanceData,
					ScaledMass,
					ScaledInertia,
					Transforms[TransformGroupIndex],
					DynamicState[TransformGroupIndex],
					static_cast<int16>(Parameters.CollisionGroup),
					CollisionParticlesPerObjectFraction);

				// initialize anchoring information if available 
				Handle->SetIsAnchored(AnchoringFacade.IsAnchored(TransformGroupIndex));

				if (Parameters.EnableClustering)
				{
					Handle->SetClusterGroupIndex(Parameters.ClusterGroupIndex);
				}

				// #BGTODO - non-updating parameters - remove lin/ang drag arrays and always query material if this stays a material parameter
				Chaos::FChaosPhysicsMaterial* SolverMaterial = RigidsSolver->GetSimMaterials().Get(Parameters.PhysicalMaterialHandle.InnerHandle);
				if (SolverMaterial)
				{
					Handle->SetLinearEtherDrag(SolverMaterial->LinearEtherDrag);
					Handle->SetAngularEtherDrag(SolverMaterial->AngularEtherDrag);
				}

				const Chaos::FShapesArray& Shapes = Handle->ShapesArray();
				for (const TUniquePtr<Chaos::FPerShapeData>& Shape : Shapes)
				{
					Shape->SetMaterial(Parameters.PhysicalMaterialHandle);
				}
			}
		},true);

		// Enable the particle (register with graph, add to broadphase, etc)
		// NOTE: Cannot be called in parallel (do not move this to the above loop)
		// @todo(chaos): can this be done at the end of the function once we know 
		// whether we're asleep etc.? (but see calls to UpdateGeometryCollectionViews)
		for (FClusterHandle* ClusterHandle : SolverParticleHandles)
		{
			if ((ClusterHandle != nullptr) && !ClusterHandle->Disabled())
			{
				RigidsSolver->GetEvolution()->EnableParticle(ClusterHandle);

				// Enable the BVH on the collision (if there are lots of shapes)
				if (ClusterHandle->GetGeometry() != nullptr)
				{
					if (const Chaos::FImplicitObjectUnion* Union = ClusterHandle->GetGeometry()->GetObject<Chaos::FImplicitObjectUnion>())
					{
						const_cast<Chaos::FImplicitObjectUnion*>(Union)->SetAllowBVH(true);
						const_cast<Chaos::FImplicitObjectUnion*>(Union)->SetLocked(true);
					}
					else if (const Chaos::FImplicitObjectUnion* UnionClustered = ClusterHandle->GetGeometry()->GetObject<Chaos::FImplicitObjectUnionClustered>())
					{
						const_cast<Chaos::FImplicitObjectUnion*>(UnionClustered)->SetAllowBVH(true);
						const_cast<Chaos::FImplicitObjectUnion*>(UnionClustered)->SetLocked(true);
					}
				}

			}
		}

		for(FFieldSystemCommand& Cmd : Parameters.InitializationCommands)
		{
			if(Cmd.MetaData.Contains(FFieldSystemMetaData::EMetaType::ECommandData_ProcessingResolution))
			{
				Cmd.MetaData.Remove(FFieldSystemMetaData::EMetaType::ECommandData_ProcessingResolution);
			}

			FFieldSystemMetaDataProcessingResolution* ResolutionData = new FFieldSystemMetaDataProcessingResolution(EFieldResolutionType::Field_Resolution_Maximum);

			Cmd.MetaData.Add(FFieldSystemMetaData::EMetaType::ECommandData_ProcessingResolution, TUniquePtr<FFieldSystemMetaDataProcessingResolution>(ResolutionData));
			RigidsSolver->GetGeometryCollectionPhysicsProxiesField_Internal().Add(this);
			FFieldData& FieldData = GetOrCreateFieldData_Internal();
			FieldData.Commands.Add(Cmd);
		}
		Parameters.InitializationCommands.Empty();
		FieldParameterUpdateCallback(RigidsSolver, false);

		if (Parameters.InitialVelocityType == EInitialVelocityTypeEnum::Chaos_Initial_Velocity_User_Defined)
		{
			FGeometryDynamicCollection::FInitialVelocityFacade InitialVelocityFacade = DynamicCollection.GetInitialVelocityFacade();
			check(InitialVelocityFacade.IsValid());
			// A previous implementation of this went wide on this loop.  The general 
			// rule of thumb for parallelization is that each thread needs at least
			// 1000 operations in order to overcome the expense of threading.  I don't
			// think that's generally going to be the case here...
			for (int32 ParticleIndex = 0; ParticleIndex < NumEffectiveParticles; ++ParticleIndex)
			{
				if (Chaos::TPBDRigidParticleHandle<Chaos::FReal, 3>* Handle = SolverParticleHandles[ParticleIndex])
				{
					const int32 TransformGroupIndex = FromParticleToTransformIndex[ParticleIndex];
					if (DynamicState[TransformGroupIndex] == (int32)EObjectStateTypeEnum::Chaos_Object_Dynamic)
					{
						Handle->SetV(InitialVelocityFacade.InitialLinearVelocityAttribute[TransformGroupIndex]);
						Handle->SetW(InitialVelocityFacade.InitialAngularVelocityAttribute[TransformGroupIndex]);
					}
				}
			}
		}

		// #BG Temporary - don't cluster when playing back. Needs to be changed when kinematics are per-proxy to support
		// kinematic to dynamic transition for clusters.
		if (Parameters.EnableClustering)// && Parameters.CacheType != EGeometryCollectionCacheType::Play)
		{
			// "RecursiveOrder" means bottom up - children come before their parents.
			const TArray<int32> RecursiveOrder = GeometryCollectionAlgo::ComputeRecursiveOrder(*RestCollection);

			// Propagate simulated particle flags up the hierarchy from children 
			// to their parents, grandparents, etc...
			TArray<bool> SubTreeContainsSimulatableParticle;
			SubTreeContainsSimulatableParticle.SetNumZeroed(RecursiveOrder.Num());
			for (const int32 TransformGroupIndex : RecursiveOrder)
			{
				if (SimulatableParticles[TransformGroupIndex] && !RestCollection->IsClustered(TransformGroupIndex))
				{
					// Rigid node
					const int32 ParticleIndex = FromTransformToParticleIndex[TransformGroupIndex];
					SubTreeContainsSimulatableParticle[TransformGroupIndex] = (ParticleIndex != INDEX_NONE && SolverParticleHandles[ParticleIndex] != nullptr) || !EffectiveParticles[TransformGroupIndex];
				}
				else
				{
					// Cluster parent
					PhysicsThreadCollection.IterateThroughChildren(TransformGroupIndex, [&](int32 ChildIndex)
					{
						if (SubTreeContainsSimulatableParticle.IsValidIndex(ChildIndex) && SubTreeContainsSimulatableParticle[ChildIndex])
						{
							SubTreeContainsSimulatableParticle[TransformGroupIndex] = true;
							return false;
						}
						return true;
					});
				}
			}

			TArray<Chaos::TPBDRigidClusteredParticleHandle<Chaos::FReal, 3>*> ClusterHandles;
			// Ryan - It'd be better to batch allocate cluster particles ahead of time,
			// but if ClusterHandles is empty, then new particles will be allocated
			// on the fly by TPBDRigidClustering::CreateClusterParticle(), which 
			// needs to work before this does...
			//ClusterHandles = GetSolver()->GetEvolution()->CreateClusteredParticles(NumClusters);

			int32 ClusterHandlesIndex = 0;
			TArray<Chaos::TPBDRigidParticleHandle<Chaos::FReal, 3>*> RigidChildren;
			TArray<int32> RigidChildrenTransformGroupIndex;
			for (const int32 TransformGroupIndex : RecursiveOrder)
			{
				// Don't construct particles for branches of the hierarchy that  
				// don't contain any simulated particles.
				if (!SubTreeContainsSimulatableParticle[TransformGroupIndex] || !EffectiveParticles[TransformGroupIndex])
				{
					continue;
				}

				RigidChildren.Reset(NumTransforms);
				RigidChildrenTransformGroupIndex.Reset(NumTransforms);

				float ParentMass = 0.0f;
				Chaos::FMatrix33 ParentInertia = Chaos::FMatrix33(0);

				if (DynamicCollection.HasChildren(TransformGroupIndex))
				{
					Masses[TransformGroupIndex] = 0;
					Inertias[TransformGroupIndex] = Chaos::FVec3(0.0);
				}

				Chaos::FMatrix33 FullInertia = Chaos::FMatrix33(0);
				PhysicsThreadCollection.IterateThroughChildren(TransformGroupIndex, [&](int32 ChildIndex)
					{
						Masses[TransformGroupIndex] += Masses[ChildIndex];
						const Chaos::FMatrix33 ChildWorldSpaceI = Chaos::Utilities::ComputeWorldSpaceInertia(Transforms[ChildIndex].GetRotation(), Inertias[ChildIndex]);
						FullInertia += ChildWorldSpaceI;
						Inertias[TransformGroupIndex] = FullInertia.GetDiagonal();

						if (EffectiveParticles[ChildIndex])
						{
							if (Chaos::TPBDRigidClusteredParticleHandle<Chaos::FReal, 3>*Handle = SolverParticleHandles[FromTransformToParticleIndex[ChildIndex]])
							{
								RigidChildren.Add(Handle);
								RigidChildrenTransformGroupIndex.Add(ChildIndex);
							}
						}
						return true;
					});

				const int32 ParticleIndex = FromTransformToParticleIndex[TransformGroupIndex];
				if (ParticleIndex != INDEX_NONE && SolverParticleHandles[ParticleIndex] == nullptr)
				{
					if (ReportTooManyChildrenNum >= 0 && RigidChildren.Num() > ReportTooManyChildrenNum)
					{
						UE_LOGF(LogChaos, Warning, "Too many children (%d) in a single cluster:%ls", 
							RigidChildren.Num(), *Parameters.Name);
					}

					Chaos::FClusterCreationParameters CreationParameters;
					CreationParameters.ClusterParticleHandle = ClusterHandles.Num() ? ClusterHandles[ClusterHandlesIndex++] : nullptr;
					CreationParameters.Scale = Parameters.WorldTransform.GetScale3D();

					// Hook the handle up with the GT particle
					FParticle* GTParticle = GTParticles[ParticleIndex].Get();
					Chaos::FUniqueIdx ExistingIndex;
					if (GTParticle != nullptr)
					{
						ExistingIndex = GTParticle->UniqueIdx();
					}
					else
					{
						ExistingIndex = UniqueIdxs[ParticleIndex];
					}

					Chaos::FPBDRigidClusteredParticleHandle* Handle = nullptr;
					if (RigidChildren.Num())
					{
						Handle = BuildClusters_Internal(TransformGroupIndex, RigidChildren, RigidChildrenTransformGroupIndex, CreationParameters, &ExistingIndex);
					}
					else
					{
						Handle = BuildNonClusters_Internal(TransformGroupIndex, RigidsSolver, Masses[TransformGroupIndex], Inertias[TransformGroupIndex], &ExistingIndex);
					}
					if (GTParticle != nullptr)
					{
						Handle->GTGeometryParticle() = GTParticle;
					}

#if CHAOS_DEBUG_NAME
					TSharedPtr<FString, ESPMode::ThreadSafe> ParticleName = MakeShared<FString, ESPMode::ThreadSafe>(FString::Printf(TEXT("%s-%d"), *Parameters.Name, TransformGroupIndex));
					Handle->SetDebugName(ParticleName);
#endif

					int32 RigidChildrenIdx = 0;
					for(const int32 ChildTransformIndex : RigidChildrenTransformGroupIndex)
					{
						if (FromTransformToParticleIndex[ChildTransformIndex] != INDEX_NONE)
						{
							SolverClusterID[FromTransformToParticleIndex[ChildTransformIndex]] = RigidChildren[RigidChildrenIdx++]->CastToClustered()->ClusterIds().Id;
						}
					}
					SolverClusterID[ParticleIndex] = Handle->ClusterIds().Id;

					SolverClusterHandles[ParticleIndex] = Handle;
					SolverParticleHandles[ParticleIndex] = Handle;
					HandleToTransformGroupIndex.Add(Handle, TransformGroupIndex);
					Handle->SetPhysicsProxy(this);

					// Dirty for SQ
					RigidsSolver->GetEvolution()->DirtyParticle(*Handle);

					// If we're not simulating we would normally not write any results back to the game thread.
					// This will force a single write in this case because we've updated the transform on the cluster
					// and it should be updated on the game thread also
					// #TODO Consider building this information at edit-time / offline
					if(!Parameters.Simulating)
					{
						RigidsSolver->GetEvolution()->GetParticles().MarkTransientDirtyParticle(Handle);
					}
				}
			}

			const GeometryCollection::Facades::FCollectionConnectionGraphFacade ConnectionFacade(*RestCollection);
			const bool bGenerateConnectionGraph = !ConnectionFacade.IsValid() || bGeometryCollectionAlwaysGenerateConnectionGraph || !ConnectionFacade.HasValidConnections();
			if (bGenerateConnectionGraph)
			{
				// Set cluster connectivity.  TPBDRigidClustering::CreateClusterParticle() 
				// will optionally do this, but we switch that functionality off in BuildClusters_Internal().
				for (int32 ParticleIndex = 0; ParticleIndex < NumEffectiveParticles; ++ParticleIndex)
				{
					const int32 TransformGroupIndex = FromParticleToTransformIndex[ParticleIndex];
					if (RestCollection->IsClustered(TransformGroupIndex))
					{
						if (FClusterHandle* ClusteredParticle = SolverParticleHandles[ParticleIndex])
						{
							Chaos::FClusterCreationParameters ClusterParams;
							// #todo: should other parameters be set here?  Previously, there was no parameters being sent, and it is unclear
							// where some of these parameters are defined (ie: CollisionThicknessPercent)
							ClusterParams.ConnectionMethod = Parameters.ClusterConnectionMethod;
							ClusterParams.ConnectionGraphBoundsFilteringMargin = Parameters.ConnectionGraphBoundsFilteringMargin;
						
							RigidsSolver->GetEvolution()->GetRigidClustering().GenerateConnectionGraph(ClusteredParticle, ClusterParams);
						}
					}
				}
				// if using the material & connectivity damnage model, we need to compute the surface area of each connection
				// since we do not have precise data from the tools, we are using an approximate method using the bounds of the two particles
				if (Parameters.DamageEvaluationModel == Chaos::EDamageEvaluationModel::StrainFromMaterialStrengthAndConnectivity)
				{
					for (int32 ParticleIndex = 0; ParticleIndex < NumEffectiveParticles; ++ParticleIndex)
					{
						if (FClusterHandle* ClusteredParticle = SolverParticleHandles[ParticleIndex])
						{
							for (Chaos::FConnectivityEdge& Edge : ClusteredParticle->ConnectivityEdges())
							{
								if (ensure(Edge.Sibling))
								{
									const Chaos::FRealSingle Area = AreaFromBoundingBoxOverlap(ClusteredParticle->WorldSpaceInflatedBounds(), Edge.Sibling->WorldSpaceInflatedBounds());
									Edge.SetArea(Area);
								}
							}
						}
					}
				}
			}
			else if (ConnectionFacade.IsValid())
			{
				// load connection graph from RestCollection
				// TODO: is it worth computing the valence of each edge to Reserve() the particle edge arrays?

				// Transfer connections from the Collection's ConnectionFacade over to the clustered particles
				int32 NumConnections = ConnectionFacade.NumConnections();
				for (int32 ConnectionIdx = 0; ConnectionIdx < NumConnections; ++ConnectionIdx)
				{
					TPair<int32,int32> Connection = ConnectionFacade.GetConnection(ConnectionIdx);
					checkSlow(Connection.Key != Connection.Value); // Graph should never contain self-loops

					bool bHasContactAreas = ConnectionFacade.HasContactAreas();
					float ContactArea = 0.0f; // Note: this default value should be unused
					if (bHasContactAreas)
					{
						ContactArea = ConnectionFacade.GetConnectionContactArea(ConnectionIdx);
					}

					const int32 KeyIndex = FromTransformToParticleIndex[Connection.Key];
					const int32 ValueIndex = FromTransformToParticleIndex[Connection.Value];
					if (KeyIndex != INDEX_NONE && ValueIndex != INDEX_NONE)
					{
						if (FClusterHandle* ClusteredParticle = SolverParticleHandles[KeyIndex])
						{
							if (FClusterHandle* OtherClusteredParticle = SolverParticleHandles[ValueIndex])
							{
								float EdgeStrainOrArea = bHasContactAreas ? ContactArea : (ClusteredParticle->GetInternalStrains() + OtherClusteredParticle->GetInternalStrains()) * 0.5f;
								if (!bHasContactAreas && Parameters.DamageEvaluationModel == Chaos::EDamageEvaluationModel::StrainFromMaterialStrengthAndConnectivity)
								{
									EdgeStrainOrArea = AreaFromBoundingBoxOverlap(ClusteredParticle->WorldSpaceInflatedBounds(), OtherClusteredParticle->WorldSpaceInflatedBounds());
								}
								// Add symmetric Chaos::TConnectivityEdge<Chaos::FReal> edges to each particle
								ClusteredParticle->ConnectivityEdges().Emplace(OtherClusteredParticle, EdgeStrainOrArea);
								OtherClusteredParticle->ConnectivityEdges().Emplace(ClusteredParticle, EdgeStrainOrArea);
							}
						}
					}
				}
			}
		} // end if EnableClustering
 
		const bool bStartSleeping =
				(Parameters.ObjectType == EObjectStateTypeEnum::Chaos_Object_Sleeping
			 || (Parameters.ObjectType == EObjectStateTypeEnum::Chaos_Object_Dynamic && !Parameters.StartAwake));
		if (bStartSleeping)
		{
			SetSleepingState(*RigidsSolver);
		}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		int32 HandleIndex = 0;
#endif

		// apply various features on the handles 
		const bool bEnableGravity = Parameters.EnableGravity && !DisableGeometryCollectionGravity;
		const TManagedArray<int32>& Level = PhysicsThreadCollection.GetInitialLevels().Get();
		for (int32 ParticleIndex = 0; ParticleIndex < SolverParticleHandles.Num(); ++ParticleIndex)
		{
			if (Chaos::FPBDRigidParticleHandle* Handle = SolverParticleHandles[ParticleIndex])
			{
				const bool bIsOneWayInteraction = (Parameters.OneWayInteractionLevel >= 0) && (Level[ParticleIndex] >= Parameters.OneWayInteractionLevel);

				Handle->SetGravityEnabled(bEnableGravity);
				Handle->SetGravityGroupIndex(Parameters.GravityGroupIndex);
				Handle->SetCCDEnabled(Parameters.UseCCD);
				Handle->SetMACDEnabled(Parameters.UseMACD);
				Handle->SetPartialIslandSleepAllowed(Parameters.AllowPartialIslandSleep);
				Handle->SetIterationSettings(Chaos::Private::FIterationSettings(Parameters.PositionSolverIterations, Parameters.VelocitySolverIterations, Parameters.ProjectionSolverIterations));
				Handle->SetOneWayInteraction(bIsOneWayInteraction);
				Handle->SetInertiaConditioningEnabled(Parameters.UseInertiaConditioning);
				Handle->SetLinearEtherDrag(Parameters.LinearDamping);
				Handle->SetAngularEtherDrag(Parameters.AngularDamping);
				Handle->SetInitialOverlapDepenetrationVelocity(Parameters.InitialOverlapDepenetrationVelocity);
				Handle->SetSleepThresholdMultiplier(Parameters.SleepThresholdMultiplier);
				Handle->SetGyroscopicTorqueEnabled(Parameters.bGyroscopicTorqueEnabled);
			}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			++HandleIndex;
#endif
		}

		// set the damage thresholds
		UpdateDamageThreshold_Internal();

		// call DirtyParticle to make sure the acceleration structure is up to date with all the changes happening here
		DirtyAllParticles(*RigidsSolver);

		if (bHasBuiltGeometryOnPT || RemoveImplicitsInDynamicCollections(Parameters))
		{
			// The Implicits attributes from the Dynamic Collection are just used for geometry initialization, after they can be removed and so free some memory.
			PhysicsThreadCollection.RemoveAttribute(FGeometryDynamicCollection::ImplicitsAttribute, FTransformCollection::TransformGroup);
		}
	} // end if simulating...

}

void FGeometryCollectionPhysicsProxy::CreateChildrenGeometry_Internal()
{
	using namespace GeometryCollectionPhysicsProxy::Private;

	if (!bHasBuiltGeometryOnPT)
	{
		const FGeometryCollection* RestCollection = Parameters.RestCollectionShared.Get();

		if (Parameters.Simulating && ensure(RestCollection))
		{
			if (RemoveImplicitsInDynamicCollections(Parameters))
			{
				PhysicsThreadCollection.AddAttribute<Chaos::FImplicitObjectPtr>(FGeometryDynamicCollection::ImplicitsAttribute, FTransformCollection::TransformGroup);
				PhysicsThreadCollection.CopyAttribute(*RestCollection, FGeometryDynamicCollection::ImplicitsAttribute, FTransformCollection::TransformGroup);
			}

			check(NumTransforms == PhysicsThreadCollection.NumElements(FTransformCollection::TransformGroup));
			const TManagedArray<FTransform>& MassToLocal = RestCollection->GetAttribute<FTransform>(MassToLocalAttributeName, FTransformCollection::TransformGroup);
			const TManagedArray<Chaos::FImplicitObjectPtr>* Implicits = PhysicsThreadCollection.FindAttribute<Chaos::FImplicitObjectPtr>(FGeometryDynamicCollection::ImplicitsAttribute, FTransformCollection::TransformGroup);

			using FSimplicialUniquePtr = TUniquePtr<FCollisionStructureManager::FSimplicial>;
			const TManagedArray<FSimplicialUniquePtr>* Simplicials = nullptr;
			if (Parameters.bUseSimplicialsWhenAvailable)
			{
				Simplicials = RestCollection->FindAttribute<FSimplicialUniquePtr>(FGeometryDynamicCollection::SimplicialsAttribute, FTransformCollection::TransformGroup);
			}

			// In PushToPhysicsState, we're going to compute a relative transform from Parameters.PrevWorldTransform
			// to a particle's current world transform to get its relative transform. Then, that relative transform is
			// applied on top of the new Parameters.WorldTransform to get the final world transform. This is problematic
			// if the particle is initialized with some Parameters.WorldTransform because then the particle's world transform
			// will have Parameters.WorldTransform baked in but Parameters.PrevWorldTransform will be an identity. Then
			// when we go to set the kinematic target we will then effectively be applying Parameters.WorldTransform twice.
			Parameters.PrevWorldTransform = Parameters.WorldTransform;

			TArray<FTransform> Transform;
			GeometryCollectionAlgo::Private::GlobalMatrices(PhysicsThreadCollection, Transform);

			for (int32 ParticleIndex = 0; ParticleIndex < NumEffectiveParticles; ParticleIndex++)
			{
				const int32 TransformIndex = FromParticleToTransformIndex[ParticleIndex];
				if (TransformIndex != Parameters.InitialRootIndex)
				{
					Chaos::TPBDGeometryCollectionParticleHandle<Chaos::FReal, 3>* Handle = static_cast<Chaos::TPBDGeometryCollectionParticleHandle<Chaos::FReal, 3>*>(SolverParticleHandles[ParticleIndex]);
					if (ensure(Handle) && Handle->GetGeometry() == nullptr)
					{
						const FTransform WorldTransform = MassToLocal[TransformIndex] * Transform[TransformIndex] * Parameters.WorldTransform;
						const FCollisionStructureManager::FSimplicial* Simplicial = Simplicials ? (*Simplicials)[TransformIndex].Get() : nullptr;
						SetImplicitToPTParticles(Handle, Parameters.Shared, Simplicial, Implicits? (*Implicits)[TransformIndex]: nullptr, ShapeFilterData, ShapeInstanceData, WorldTransform, CollisionParticlesPerObjectFraction);
						check(Handle->GetGeometry() != nullptr);
					}
				}
			}

			if (Chaos::FPhysicsSolver* RBDSolver = GetSolver<Chaos::FPhysicsSolver>())
			{
				// call DirtyParticle to make sure the acceleration structure is up to date with all the changes happening here
				DirtyAllParticles(*RBDSolver);
			}
		}
		bHasBuiltGeometryOnPT = true;
		// The Implicits attributes from the Dynamic Collection are just used for geometry initialization, after they can be removed and so free some memory.
		PhysicsThreadCollection.RemoveAttribute(FGeometryDynamicCollection::ImplicitsAttribute, FTransformCollection::TransformGroup);
	}
}

void FGeometryCollectionPhysicsProxy::SyncParticles_External()
{
	ExecuteOnPhysicsThread(*this, [this]()
	{
		int32 ParticlNum = GTParticles.Num();
		for (int32 Index = 0; Index < ParticlNum; ++Index)
		{
			FParticle* GTParticle = GTParticles[Index].Get();
			check(GTParticle != nullptr);
			FClusterHandle* Handle = SolverParticleHandles[Index];
			if (Handle != nullptr)
			{
				Handle->GTGeometryParticle() = GTParticle;
			}
		}
	});
}


int32 ReportNoLevelsetCluster = 0;
FAutoConsoleVariableRef CVarReportNoLevelsetCluster(TEXT("p.gc.ReportNoLevelsetCluster"), ReportNoLevelsetCluster, TEXT("Report any cluster objects without levelsets"));

int32 GlobalMaxSimulatedLevel = 100;
FAutoConsoleVariableRef CVarGlobalMaxSimulatedLevel(TEXT("p.gc.GlobalMaxSimulatedLevel"), GlobalMaxSimulatedLevel, TEXT("Allow to set the Global Maximum Simulated Level for Geoemtry Collection. The min between the MaxSimulatedLevel and the GlobalMaxSimulatedLevel will be used. "));


DECLARE_CYCLE_STAT(TEXT("FGeometryCollectionPhysicsProxy::BuildClusters"), STAT_BuildClusters, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("FGeometryCollectionPhysicsProxy::BuildClusters:GlobalMatrices"), STAT_BuildClustersGlobalMatrices, STATGROUP_Chaos);

float FGeometryCollectionPhysicsProxy::ComputeMaterialBasedDamageThreshold_Internal(int32 TransformIndex) const
{
	float DamageThreshold = TNumericLimits<float>::Max();
	if (Parameters.DamageModel == EDamageModelTypeEnum::Chaos_Damage_Model_Material_Strength_And_Connectivity_DamageThreshold)
	{
		const int32 ParticleIndex = FromTransformToParticleIndex[TransformIndex];
		if (ensure(SolverParticleHandles.IsValidIndex(ParticleIndex)))
		{
			if (Chaos::FPBDRigidClusteredParticleHandle* ClusteredParticle = SolverParticleHandles[ParticleIndex])
			{
				DamageThreshold = ComputeMaterialBasedDamageThreshold_Internal(*ClusteredParticle);
			}
		}
	}

	return DamageThreshold;
}

float FGeometryCollectionPhysicsProxy::ComputeMaterialBasedDamageThreshold_Internal(Chaos::FPBDRigidClusteredParticleHandle& ClusteredParticle) const
{
	float DamageThreshold = TNumericLimits<float>::Max();
	float ComputedConnectionArea = 0;

	// store in a local variable outisde of the loop , to make sure the compiler knows we are knot going to change it while in the loop
	const int32 ComputationMode = GeometryCollectionAreaBasedDamageThresholdMode;

	const Chaos::FConnectivityEdgeArray& ConnectivityEdges = ClusteredParticle.ConnectivityEdges();
	for (const Chaos::FConnectivityEdge& Connection : ConnectivityEdges)
	{
		switch (ComputationMode)
		{
		case 0: // sum of areas
		case 3: // average of areas ( we'll divide by the number of connection after the loop )
			ComputedConnectionArea += Connection.GetArea();
			break;
		case 1: // max of areas
			ComputedConnectionArea = FMath::Max(ComputedConnectionArea, Connection.GetArea());
			break;
		case 2: // min of areas
			ComputedConnectionArea = FMath::Min(ComputedConnectionArea, Connection.GetArea());
			break;
		}
	}
	// if average we need to didvide by count
	if (ComputationMode == 3 && ConnectivityEdges.Num() > 0)
	{
		ComputedConnectionArea /= (float)ConnectivityEdges.Num();
	}

	if (ComputedConnectionArea > 0)
	{
		// This model compute the total surface are from the connections and compute the maximum force 
		if (const Chaos::FChaosPhysicsMaterial* ChaosMaterial = Parameters.PhysicalMaterialHandle.Get())
		{
			// force unit is Kg.cm/s2 here
			const float ForceThreshold = ChaosMaterial->Strength.TensileStrength * ComputedConnectionArea;
			DamageThreshold = ForceThreshold;
		}
	}
	return DamageThreshold;
}

float FGeometryCollectionPhysicsProxy::ComputeUserDefinedDamageThreshold_Internal(int32 TransformIndex) const
{
	float DamageThreshold = TNumericLimits<float>::Max();

	const int32 LevelOffset = Parameters.bUsePerClusterOnlyDamageThreshold ? 0 : -1;
	const int32 Level = FMath::Clamp(CalculateHierarchyLevel(PhysicsThreadCollection, TransformIndex) + LevelOffset, 0, INT_MAX);
	if (Level >= FMath::Min(Parameters.MaxClusterLevel, FMath::Min(GlobalMaxSimulatedLevel, Parameters.MaxSimulatedLevel)))
	{
		DamageThreshold = TNumericLimits<float>::Max();
	}
	else if (Parameters.DamageModel == EDamageModelTypeEnum::Chaos_Damage_Model_UserDefined_Damage_Threshold)
	{
		const int32 ParticleIndex = FromTransformToParticleIndex[TransformIndex];
		// we don't test for the damage model because this is used
		if (ParticleIndex != INDEX_NONE && Parameters.bUseSizeSpecificDamageThresholds)
		{
			// bounding box volume is used as a fallback to find specific size if the relative size if not available
			// ( May happen with older GC )
			const FClusterHandle* Handle = SolverParticleHandles[ParticleIndex];
			FBox LocalBoundingBox;
			if (Handle && Handle->HasBounds())
			{
				const Chaos::FAABB3& ImplicitBoundingBox = Handle->LocalBounds();
				LocalBoundingBox = FBox(ImplicitBoundingBox.Min(), ImplicitBoundingBox.Max());
			}

			const FSharedSimulationSizeSpecificData& SizeSpecificData = GetSizeSpecificData(Parameters.Shared.SizeSpecificData, *Parameters.RestCollectionShared, TransformIndex, LocalBoundingBox);
			DamageThreshold = SizeSpecificData.DamageThreshold;
		}
		else
		{
			const int32 NumThresholds = Parameters.DamageThreshold.Num();
			const float DefaultDamage = NumThresholds > 0 ? Parameters.DamageThreshold[NumThresholds - 1] : 0.f;
			DamageThreshold = Level < NumThresholds ? Parameters.DamageThreshold[Level] : DefaultDamage;
		}

		if (Parameters.bUseMaterialDamageModifiers)
		{
			if (const Chaos::FChaosPhysicsMaterial* Material = Parameters.PhysicalMaterialHandle.Get())
			{
				DamageThreshold *= Material->DamageModifier.DamageThresholdMultiplier;
			}
		}
	}

	return DamageThreshold;
}

float FGeometryCollectionPhysicsProxy::AdjustMassForScale(float Mass) const
{
	const FVector WorldScale = Parameters.WorldTransform.GetScale3D();
	float MassScale = (float)(WorldScale.X * WorldScale.Y * WorldScale.Z);
	MassScale *= Parameters.MaterialOverrideMassScaleMultiplier;
	return FMath::Abs(Mass * MassScale);
}

Chaos::FVec3f FGeometryCollectionPhysicsProxy::AdjustInertiaForScale(const Chaos::FVec3f& Inertia) const
{
	const FVector3f WorldScale(Parameters.WorldTransform.GetScale3D());
	const FVector3f WorldScaledInertia = Chaos::Utilities::ScaleInertia<float>(Inertia, WorldScale, true);
	return WorldScaledInertia * Parameters.MaterialOverrideMassScaleMultiplier;
}

Chaos::TPBDGeometryCollectionParticleHandle<Chaos::FReal, 3>* FGeometryCollectionPhysicsProxy::BuildNonClusters_Internal(const uint32 CollectionClusterIndex, Chaos::FPBDRigidsSolver* RigidsSolver, float Mass, Chaos::FVec3f Inertia, const Chaos::FUniqueIdx* ExistingIndex)
{
	using namespace GeometryCollectionPhysicsProxy::Private;

	FGeometryDynamicCollection& DynamicCollection = PhysicsThreadCollection;
	TManagedArray<uint8>& DynamicState = DynamicCollection.DynamicState;
	const TManagedArray<FTransform>& MassToLocal = Parameters.RestCollectionShared->GetAttribute<FTransform>(MassToLocalAttributeName, FTransformCollection::TransformGroup);
	const TManagedArray<Chaos::FImplicitObjectPtr>* Implicits = DynamicCollection.FindAttribute<Chaos::FImplicitObjectPtr>(FGeometryDynamicCollection::ImplicitsAttribute, FTransformCollection::TransformGroup);

	using FSimplicialUniquePtr = TUniquePtr<FCollisionStructureManager::FSimplicial>;
	const TManagedArray<FSimplicialUniquePtr>* Simplicials = nullptr;
	if (Parameters.bUseSimplicialsWhenAvailable)
	{
		Simplicials = Parameters.RestCollectionShared->FindAttribute<FSimplicialUniquePtr>(DynamicCollection.SimplicialsAttribute, FTransformCollection::TransformGroup);
	}
	Chaos::Facades::FCollectionAnchoringFacade AnchoringFacade(DynamicCollection);

	//If we are a root particle use the world transform, otherwise set the relative transform
	const FTransform CollectionSpaceTransform = GeometryCollectionAlgo::Private::GlobalMatrix(DynamicCollection, CollectionClusterIndex);
	const Chaos::TRigidTransform<Chaos::FReal, 3> ParticleTM = MassToLocal[CollectionClusterIndex] * CollectionSpaceTransform * Parameters.WorldTransform;

	// Gather unique indices from GT to pass into PT handle creation
	TArray<Chaos::FUniqueIdx> UniqueIndices;
	UniqueIndices.Add(*ExistingIndex);

	// Add entries into simulation array
	RigidsSolver->GetEvolution()->ReserveParticles(1);
	TArray<Chaos::TPBDGeometryCollectionParticleHandle<Chaos::FReal, 3>*> Handles = RigidsSolver->GetEvolution()->CreateGeometryCollectionParticles(1, UniqueIndices.GetData());

	Chaos::TPBDGeometryCollectionParticleHandle<Chaos::FReal, 3>* Handle = Handles[0];

	Handle->SetPhysicsProxy(this);
	const int32 ParticleIndex = FromTransformToParticleIndex[CollectionClusterIndex];
	check(ParticleIndex != INDEX_NONE);
	SolverParticleHandles[ParticleIndex] = Handle;
	HandleToTransformGroupIndex.Add(Handle, CollectionClusterIndex);

	// We're on the physics thread here but we've already set up the GT particles and we're just linking here
	Handle->GTGeometryParticle() = GTParticles[ParticleIndex].Get();

	check(SolverParticleHandles[ParticleIndex]->GetParticleType() == Handle->GetParticleType());
	RigidsSolver->GetEvolution()->RegisterParticle(Handle);


	if (AnchoringFacade.IsAnchored(CollectionClusterIndex))
	{
		if (Handle->ObjectState() == Chaos::EObjectStateType::Static)
		{
			DynamicState[CollectionClusterIndex] = (uint8)EObjectStateTypeEnum::Chaos_Object_Static;
		}
		else
		{
			DynamicState[CollectionClusterIndex] = (uint8)EObjectStateTypeEnum::Chaos_Object_Kinematic;
		}
	}

	check(Parameters.RestCollectionShared);

	const bool bUseImplicit = (Implicits != nullptr) && (BuildGeometryForChildrenOnPT(Parameters) || (CollectionClusterIndex == Parameters.InitialRootIndex));

	// NOTE: The particle creation call above (CreateClusterParticle) activates the particle, which does various things including adding
	// it to the constraint graph. It seems a bit dodgy to be completely changing what the particle is after it has been enabled. Maybe we should fix this.
	PopulateSimulatedParticle(
		Handle,
		Parameters.Shared,
		Simplicials? (*Simplicials)[CollectionClusterIndex].Get(): nullptr,
		bUseImplicit? (*Implicits)[CollectionClusterIndex] : nullptr,
		ShapeFilterData, 
		ShapeInstanceData, 
		Mass,
		Inertia,
		ParticleTM,
		static_cast<uint8>(DynamicState[CollectionClusterIndex]),
		static_cast<int16>(Parameters.CollisionGroup),
		CollisionParticlesPerObjectFraction);


	// initialize anchoring information if available 
	Handle->SetIsAnchored(AnchoringFacade.IsAnchored(CollectionClusterIndex));

	if (Parameters.EnableClustering)
	{
		Handle->SetClusterGroupIndex(Parameters.ClusterGroupIndex);
	}

	// #BGTODO This will not automatically update - material properties should only ever exist in the material, not in other arrays
	const Chaos::FChaosPhysicsMaterial* CurMaterial = static_cast<Chaos::FPBDRigidsSolver*>(Solver)->GetSimMaterials().Get(Parameters.PhysicalMaterialHandle.InnerHandle);
	if (CurMaterial)
	{
		Handle->SetLinearEtherDrag(CurMaterial->LinearEtherDrag);
		Handle->SetAngularEtherDrag(CurMaterial->AngularEtherDrag);
	}

	const Chaos::FShapesArray& Shapes = Handle->ShapesArray();
	for (const TUniquePtr<Chaos::FPerShapeData>& Shape : Shapes)
	{
		Shape->SetMaterial(Parameters.PhysicalMaterialHandle);
	}

	const FTransform ParentTransform = GeometryCollectionAlgo::Private::GlobalMatrix(DynamicCollection, CollectionClusterIndex);

	// Populate bounds as we didn't pass a shared implicit to PopulateSimulatedParticle this will have been skipped, now that we have the full cluster we can build it
	if (Handle->GetGeometry() && Handle->GetGeometry()->HasBoundingBox())
	{
		Handle->SetHasBounds(true);
		Handle->SetLocalBounds(Handle->GetGeometry()->BoundingBox());
		const Chaos::FRigidTransform3 Xf(Handle->GetX(), Handle->GetR());
		Handle->UpdateWorldSpaceState(Xf, Chaos::FVec3(0));

		static_cast<Chaos::FPBDRigidsSolver*>(Solver)->GetEvolution()->DirtyParticle(*Handle);
	}

	return Handle;
}



Chaos::FPBDRigidClusteredParticleHandle*
FGeometryCollectionPhysicsProxy::BuildClusters_Internal(
	const uint32 CollectionClusterIndex,
	TArray<Chaos::FPBDRigidParticleHandle*>& ChildHandles,
	const TArray<int32>& ChildTransformGroupIndices,
	const Chaos::FClusterCreationParameters & ClusterParameters,
	const Chaos::FUniqueIdx* ExistingIndex)
{
	SCOPE_CYCLE_COUNTER(STAT_BuildClusters);

	check(CollectionClusterIndex != INDEX_NONE);
	check(ChildHandles.Num() != 0);

	FGeometryDynamicCollection& DynamicCollection = PhysicsThreadCollection;
	TManagedArray<uint8>& DynamicState = DynamicCollection.DynamicState;
	const TManagedArray<FTransform>& MassToLocal = Parameters.RestCollectionShared->GetAttribute<FTransform>(MassToLocalAttributeName, FTransformCollection::TransformGroup);
	const TManagedArray<Chaos::FImplicitObjectPtr>* Implicits = DynamicCollection.FindAttribute<Chaos::FImplicitObjectPtr>(FGeometryDynamicCollection::ImplicitsAttribute, FTransformCollection::TransformGroup);

	//If we are a root particle use the world transform, otherwise set the relative transform
	const FTransform CollectionSpaceTransform = GeometryCollectionAlgo::Private::GlobalMatrix(DynamicCollection, CollectionClusterIndex);
	const Chaos::TRigidTransform<Chaos::FReal, 3> ParticleTM = MassToLocal[CollectionClusterIndex] * CollectionSpaceTransform * Parameters.WorldTransform;

	//create new cluster particle
	//The reason we need to pass in a mass orientation override is as follows:
	//Consider a pillar made up of many boxes along the Y-axis. In this configuration we could generate a proxy pillar along the Y with identity rotation.
	//Now if we instantiate the pillar and rotate it so that it is along the X-axis, we would still like to use the same pillar proxy.
	//Since the mass orientation is computed in world space in both cases we'd end up with a diagonal inertia matrix and identity rotation that looks like this: [big, small, big] or [small, big, big].
	//Because of this we need to know how to rotate collision particles and geometry to match with original computation. If it was just geometry we could transform it before passing, but we need collision particles as well
	Chaos::FClusterCreationParameters ClusterCreationParameters = ClusterParameters;
	// connectivity is taken care outside, no need to generate it here
	ClusterCreationParameters.bGenerateConnectionGraph = false;
	// fix... ClusterCreationParameters.CollisionParticles = Simplicials[CollectionClusterIndex];
	ClusterCreationParameters.ConnectionMethod = Parameters.ClusterConnectionMethod;
	if (ClusterCreationParameters.CollisionParticles)
	{
		const Chaos::FReal NumCollisionParticles = static_cast<Chaos::FReal>(ClusterCreationParameters.CollisionParticles->Size());
		const int32 ClampedCollisionParticlesSize = 
			FMath::TruncToInt32(Chaos::FReal(FMath::Max(0, FMath::Min(NumCollisionParticles * CollisionParticlesPerObjectFraction, NumCollisionParticles))));
		ClusterCreationParameters.CollisionParticles->Resize(ClampedCollisionParticlesSize);
	}
	Chaos::Facades::FCollectionAnchoringFacade AnchoringFacade(DynamicCollection);
	ClusterCreationParameters.bIsAnchored = AnchoringFacade.IsAnchored(CollectionClusterIndex);
	
	TArray<Chaos::TPBDRigidParticleHandle<Chaos::FReal, 3>*> ChildHandlesCopy(ChildHandles);

	// Construct an active cluster particle, disable children, derive M and I from children:
	Chaos::TPBDRigidClusteredParticleHandle<Chaos::FReal, 3>* Parent =
		static_cast<Chaos::FPBDRigidsSolver*>(Solver)->GetEvolution()->GetRigidClustering().CreateClusterParticle(
			(!DynamicCollection.GetHasParent(CollectionClusterIndex)) ? Parameters.ClusterGroupIndex : 0, 
			MoveTemp(ChildHandlesCopy),
			ClusterCreationParameters,
			Implicits? (*Implicits)[CollectionClusterIndex]: nullptr, // union from children if null
			&ParticleTM,
			ExistingIndex
			);

	if (ReportNoLevelsetCluster && Parent->GetGeometry())
	{
		//ensureMsgf(false, TEXT("Union object generated for cluster"));
		UE_LOGF(LogChaos, Warning, "Union object generated for cluster:%ls", *Parameters.Name);
	}

	if (Parent->InvM() == 0.0)
	{
		if (Parent->ObjectState() == Chaos::EObjectStateType::Static)
		{
			DynamicState[CollectionClusterIndex] = (uint8)EObjectStateTypeEnum::Chaos_Object_Static;
		}
		else //if (Particles.ObjectState(NewSolverClusterID) == Chaos::EObjectStateType::Kinematic)
		{
			DynamicState[CollectionClusterIndex] = (uint8)EObjectStateTypeEnum::Chaos_Object_Kinematic;
		}
	}

	check(Parameters.RestCollectionShared);
	const TManagedArray<float>& Mass = Parameters.RestCollectionShared->GetAttribute<float>(MassAttributeName, FTransformCollection::TransformGroup);
	const TManagedArray<FVector3f>& InertiaTensor = Parameters.RestCollectionShared->GetAttribute<FVector3f>(InertiaTensorAttributeName, FTransformCollection::TransformGroup);

	const float ScaledMass = AdjustMassForScale(Mass[CollectionClusterIndex]);
	const Chaos::FVec3f ScaledInertia = AdjustInertiaForScale((Chaos::FVec3f)InertiaTensor[CollectionClusterIndex]);
	
	// NOTE: The particle creation call above (CreateClusterParticle) activates the particle, which does various things including adding
	// it to the constraint graph. It seems a bit dodgy to be completely changing what the particle is after it has been enabled. Maybe we should fix this.
	PopulateSimulatedParticle(
		Parent,
		Parameters.Shared, 
		nullptr, // CollisionParticles is optionally set from CreateClusterParticle()
		nullptr, // Parent->Geometry() ? Parent->Geometry() : Implicits[CollectionClusterIndex], 
		ShapeFilterData, 
		ShapeInstanceData, 
		ScaledMass,
		ScaledInertia,
		ParticleTM, 
		(uint8)DynamicState[CollectionClusterIndex],
		0, // static_cast<int16>(CollisionGroup[TransformGroupIndex])
		CollisionParticlesPerObjectFraction); // CollisionGroup

	// two-way mapping
	const int32 ParticleClusterIndex = FromTransformToParticleIndex[CollectionClusterIndex];
	check(ParticleClusterIndex != INDEX_NONE);
	SolverClusterHandles[ParticleClusterIndex] = Parent;

	// #BGTODO This will not automatically update - material properties should only ever exist in the material, not in other arrays
	const Chaos::FChaosPhysicsMaterial* CurMaterial = static_cast<Chaos::FPBDRigidsSolver*>(Solver)->GetSimMaterials().Get(Parameters.PhysicalMaterialHandle.InnerHandle);
	if(CurMaterial)
	{
		Parent->SetLinearEtherDrag(CurMaterial->LinearEtherDrag);
		Parent->SetAngularEtherDrag(CurMaterial->AngularEtherDrag);
	}

	const Chaos::FShapesArray& Shapes = Parent->ShapesArray();
	for(const TUniquePtr<Chaos::FPerShapeData>& Shape : Shapes)
	{
		Shape->SetMaterial(Parameters.PhysicalMaterialHandle);
	}

	const FTransform ParentTransform = GeometryCollectionAlgo::Private::GlobalMatrix(DynamicCollection, CollectionClusterIndex);

	int32 MinCollisionGroup = INT_MAX;
	for(int32 Idx=0; Idx < ChildHandles.Num(); Idx++)
	{
		// set the damage threshold on children as they are the one where the strain is tested when breaking 
		Chaos::FPBDRigidParticleHandle* Child = ChildHandles[Idx];

		const int32 ChildTransformGroupIndex = ChildTransformGroupIndices[Idx];
		const int32 ChildParticleIndex = FromTransformToParticleIndex[ChildTransformGroupIndex];
		check(ChildParticleIndex != INDEX_NONE);
		SolverClusterHandles[ChildParticleIndex] = Parent;

		MinCollisionGroup = FMath::Min(Child->CollisionGroup(), MinCollisionGroup);
	}
	Parent->SetCollisionGroup(MinCollisionGroup);

	// Populate bounds as we didn't pass a shared implicit to PopulateSimulatedParticle this will have been skipped, now that we have the full cluster we can build it
	if(Parent->GetGeometry() && Parent->GetGeometry()->HasBoundingBox())
	{
		Parent->SetHasBounds(true);
		Parent->SetLocalBounds(Parent->GetGeometry()->BoundingBox());
		const Chaos::FRigidTransform3 Xf(Parent->GetX(), Parent->GetR());
		Parent->UpdateWorldSpaceState(Xf, Chaos::FVec3(0));

		static_cast<Chaos::FPBDRigidsSolver*>(Solver)->GetEvolution()->DirtyParticle(*Parent);
	}

	return Parent;
}

void FGeometryCollectionPhysicsProxy::GetFilteredParticleHandles(
	TArray<Chaos::TGeometryParticleHandle<Chaos::FReal, 3>*>& Handles,
	const Chaos::FPBDRigidsSolver* RigidSolver,
	const EFieldFilterType FilterType,
	const EFieldObjectType ObjectType)
{
	Handles.SetNum(0, EAllowShrinking::No);
	if ((ObjectType == EFieldObjectType::Field_Object_All) || (ObjectType == EFieldObjectType::Field_Object_Destruction) || (ObjectType == EFieldObjectType::Field_Object_Max))
	{
		// only the local handles
		Handles.Reserve(SolverParticleHandles.Num());

		if (FilterType == EFieldFilterType::Field_Filter_Dynamic)
		{
			for (FClusterHandle* ClusterHandle : SolverParticleHandles)
			{
				if (ClusterHandle && (ClusterHandle->ObjectState() == Chaos::EObjectStateType::Dynamic))
				{
					Handles.Add(ClusterHandle);
				}
			}
		}
		else if (FilterType == EFieldFilterType::Field_Filter_Kinematic)
		{
			for (FClusterHandle* ClusterHandle : SolverParticleHandles)
			{
				if (ClusterHandle && (ClusterHandle->ObjectState() == Chaos::EObjectStateType::Kinematic))
				{
					Handles.Add(ClusterHandle);
				}
			}
		}
		else if (FilterType == EFieldFilterType::Field_Filter_Static)
		{
			for (FClusterHandle* ClusterHandle : SolverParticleHandles)
			{
				if (ClusterHandle && (ClusterHandle->ObjectState() == Chaos::EObjectStateType::Static))
				{
					Handles.Add(ClusterHandle);
				}
			}
		}
		else if (FilterType == EFieldFilterType::Field_Filter_Sleeping)
		{
			for (FClusterHandle* ClusterHandle : SolverParticleHandles)
			{
				if (ClusterHandle && (ClusterHandle->ObjectState() == Chaos::EObjectStateType::Sleeping))
				{
					Handles.Add(ClusterHandle);
				}
			}
		}
		else if (FilterType == EFieldFilterType::Field_Filter_Disabled)
		{
			for (FClusterHandle* ClusterHandle : SolverParticleHandles)
			{
				if (ClusterHandle && ClusterHandle->Disabled())
				{
					Handles.Add(ClusterHandle);
				}
			}
		}
		else if (FilterType == EFieldFilterType::Field_Filter_DynamicAndSleeping)
		{
			for (FClusterHandle* ClusterHandle : SolverParticleHandles)
			{
				if (ClusterHandle && (ClusterHandle->ObjectState() == Chaos::EObjectStateType::Dynamic || ClusterHandle->ObjectState() == Chaos::EObjectStateType::Sleeping))
				{
					Handles.Add(ClusterHandle);
				}
			}
		}
		else if (FilterType == EFieldFilterType::Field_Filter_All)
		{
			for (FClusterHandle* ClusterHandle : SolverParticleHandles)
			{
				if (ClusterHandle && (ClusterHandle->ObjectState() != Chaos::EObjectStateType::Uninitialized))
				{
					Handles.Add(ClusterHandle);
				}
			}
		}
	}
}

void FGeometryCollectionPhysicsProxy::GetRelevantParticleHandles(
	TArray<Chaos::TGeometryParticleHandle<Chaos::FReal, 3>*>& Handles,
	const Chaos::FPBDRigidsSolver* RigidSolver, 
	EFieldResolutionType ResolutionType)
{
	Handles.SetNum(0, EAllowShrinking::No);

	// only the local handles
	Handles.Reserve(SolverParticleHandles.Num());

	if (ResolutionType == EFieldResolutionType::Field_Resolution_Maximum)
	{
		for (FClusterHandle* ClusterHandle : SolverParticleHandles)
		{
			if (ClusterHandle )
			{
				Handles.Add(ClusterHandle);
			}
		}
	}
	else if (ResolutionType == EFieldResolutionType::Field_Resolution_DisabledParents)
	{
		for (FClusterHandle* ClusterHandle : SolverParticleHandles)
		{
			if (ClusterHandle && ClusterHandle->ClusterIds().Id == nullptr)
			{
				Handles.Add(ClusterHandle);
			}
		}
	}
	else if (ResolutionType == EFieldResolutionType::Field_Resolution_Minimal)
	{
		const auto& Clustering = RigidSolver->GetEvolution()->GetRigidClustering();
		const auto& ClusterMap = Clustering.GetChildrenMap();

		for (FClusterHandle* ClusterHandle : SolverParticleHandles)
		{
			if (ClusterHandle)
			{
				FClusterHandle* HandleToAdd = nullptr;
				if (ClusterHandle->Disabled())
				{
					if (FClusterHandle* ParentHandle = ClusterHandle->Parent())
					{
						bool bHasPartialClusterParent = ParentHandle->InternalCluster() && !ParentHandle->Disabled() && (ParentHandle->PhysicsProxy() == ClusterHandle->PhysicsProxy());
						if (bHasPartialClusterParent)
						{
							HandleToAdd = ParentHandle;
						}
					}
				}
				else
				{
					HandleToAdd = ClusterHandle;
				}
				if (HandleToAdd)
				{
					Handles.Add(HandleToAdd);
					if (HandleToAdd->ClusterIds().NumChildren)
					{
						if (ClusterMap.Find(HandleToAdd))
						{
							for (Chaos::FPBDRigidParticleHandle* ChildHandle : ClusterMap[HandleToAdd])
							{
								if (ChildHandle->Disabled())
								{
									Handles.Add(ChildHandle);
								}
							}
						}
					}
				}
			}
		}
	}

#if TODO_REIMPLEMENT_GET_RIGID_PARTICLES
		const Chaos::FPhysicsSolver::FParticlesType & Particles = RigidSolver->GetRigidParticles();
		if (ResolutionType == EFieldResolutionType::Field_Resolution_Minimal)
		{
			const Chaos::TArrayCollectionArray<Chaos::ClusterId>& ClusterIdArray = RigidSolver->GetRigidClustering().GetClusterIdsArray();


			//  Generate a Index mapping between the rigid body indices and 
			//  the particle indices. This allows the geometry collection to
			//  evaluate only its own ACTIVE particles + ClusterChildren
			int32 NumIndices = 0;
			Array.SetNumUninitialized(RigidBodyID.Num());
			for (int32 i = 0; i < RigidBodyID.Num(); i++)
			{
				const int32 RigidBodyIndex = RigidBodyID[i];
				if (RigidBodyIndex != INDEX_NONE && !Particles.Disabled(RigidBodyIndex)) // active bodies
				{
					Array[NumIndices] = { RigidBodyID[i],i };
					NumIndices++;
				}
				if (ClusterIdArray[RigidBodyIndex].Id != INDEX_NONE && !Particles.Disabled(ClusterIdArray[RigidBodyIndex].Id)) // children
				{
					Array[NumIndices] = { RigidBodyID[i],i };
					NumIndices++;
				}
			}
			Array.SetNum(NumIndices);
		}
		else if (ResolutionType == EFieldResolutionType::Field_Resolution_Maximum)
		{
			//  Generate a Index mapping between the rigid body indices and 
			//  the particle indices. This allows the geometry collection to
			//  evaluate only its own particles. 
			int32 NumIndices = 0;
			Array.SetNumUninitialized(RigidBodyID.Num());
			for (int32 i = 0; i < RigidBodyID.Num(); i++)
			{
				const int32 RigidBodyIndex = RigidBodyID[i];
				if (RigidBodyIndex != INDEX_NONE)
				{
					Array[NumIndices] = { RigidBodyIndex, i };
					NumIndices++;
				}
			}
			Array.SetNum(NumIndices);
		}
#endif
}

FName FGeometryCollectionPhysicsProxy::GetTransformName_External(const FGeometryCollectionItemIndex ItemIndex) const
{
	if (Parameters.RestCollectionShared)
	{
		if (!ItemIndex.IsInternalCluster())
		{
			const Chaos::FPhysicsObjectHandle PhysicsObject = GetPhysicsObjectByIndex(ItemIndex.GetTransformIndex());
			return Chaos::FPhysicsObjectInterface::GetName(PhysicsObject);
		}
	}

	// NOTE: Internal clusters don't have names
	return NAME_None;
}

void FGeometryCollectionPhysicsProxy::DisableParticles_External(TArray<int32>&& TransformGroupIndices)
{
	check(IsInGameThread());

	if (Chaos::FPhysicsSolver* RBDSolver = GetSolver<Chaos::FPhysicsSolver>())
	{
		RBDSolver->EnqueueCommandImmediate([this, RBDSolver, IndicesToDisable = MoveTemp(TransformGroupIndices)]()
			{
				for (int32 TransformIdx : IndicesToDisable)
				{
					const int32 ParticleIndex = FromTransformToParticleIndex[TransformIdx];
					check(ParticleIndex != INDEX_NONE);
					RBDSolver->GetEvolution()->DisableParticleWithRemovalEvent(SolverParticleHandles[ParticleIndex]);
					RBDSolver->GetParticles().MarkTransientDirtyParticle(SolverParticleHandles[ParticleIndex]);
				}
			});
	}
}

void FGeometryCollectionPhysicsProxy::ApplyForceAt_External(FVector Force, FVector WorldLocation)
{
	check(IsInGameThread());

	if (Chaos::FPhysicsSolver* RBDSolver = GetSolver<Chaos::FPhysicsSolver>())
	{
		RBDSolver->EnqueueCommandImmediate([this, Force, WorldLocation, RBDSolver]()
			{
				Chaos::FReal ClosestDistanceSquared = TNumericLimits<Chaos::FReal>::Max();
				Chaos::FPBDRigidClusteredParticleHandle* ClosestHandle = nullptr;

				Chaos::FRigidClustering& Clustering = RBDSolver->GetEvolution()->GetRigidClustering();
				for (Chaos::FPBDRigidClusteredParticleHandle* ClusteredHandle : Clustering.GetTopLevelClusterParents())
				{
					if (ClusteredHandle && ClusteredHandle->PhysicsProxy() == this)
					{
						if (ClusteredHandle->IsDynamic())
						{
							const Chaos::FReal DistanceSquared = (WorldLocation - ClusteredHandle->GetX()).SquaredLength();
							if (DistanceSquared < ClosestDistanceSquared)
							{
								ClosestDistanceSquared = DistanceSquared;
								ClosestHandle = ClusteredHandle;
							}
						}
					}
				}
				if (ClosestHandle)
				{
					if (ClosestHandle->IsSleeping())
					{
						RBDSolver->GetEvolution()->WakeParticle(ClosestHandle);
					}

					const Chaos::FVec3 WorldCOM = Chaos::FParticleUtilitiesXR::GetCoMWorldPosition(ClosestHandle);
					const Chaos::FVec3 WorldTorque = Chaos::FVec3::CrossProduct(WorldLocation - WorldCOM, Force);
					ClosestHandle->AddForce(Force);
					ClosestHandle->AddTorque(WorldTorque);
				}
			});
	}
}

void FGeometryCollectionPhysicsProxy::ApplyImpulseAt_External(FVector Impulse, FVector WorldLocation)
{
	check(IsInGameThread());

	if (Chaos::FPhysicsSolver* RBDSolver = GetSolver<Chaos::FPhysicsSolver>())
	{
		RBDSolver->EnqueueCommandImmediate([this, Impulse, WorldLocation, RBDSolver]()
			{
				Chaos::FReal ClosestDistanceSquared = TNumericLimits<Chaos::FReal>::Max();
				Chaos::FPBDRigidClusteredParticleHandle* ClosestHandle = nullptr;

				Chaos::FRigidClustering& Clustering = RBDSolver->GetEvolution()->GetRigidClustering();
				for (Chaos::FPBDRigidClusteredParticleHandle* ClusteredHandle : Clustering.GetTopLevelClusterParents())
				{
					if (ClusteredHandle && ClusteredHandle->PhysicsProxy() == this)
					{
						if (ClusteredHandle->IsDynamic())
						{
							const Chaos::FReal DistanceSquared = (WorldLocation - ClusteredHandle->GetX()).SquaredLength();
							if (DistanceSquared < ClosestDistanceSquared)
							{
								ClosestDistanceSquared = DistanceSquared;
								ClosestHandle = ClusteredHandle;
							}
						}
					}
				}
				if (ClosestHandle)
				{
					if (ClosestHandle->IsSleeping())
					{
						RBDSolver->GetEvolution()->WakeParticle(ClosestHandle);
					}

					const Chaos::FVec3 WorldCOM = Chaos::FParticleUtilitiesXR::GetCoMWorldPosition(ClosestHandle);
					ClosestHandle->SetLinearImpulseVelocity(ClosestHandle->LinearImpulseVelocity() + (Impulse * ClosestHandle->InvM()), false);

					const Chaos::FMatrix33 WorldInvI = Chaos::Utilities::ComputeWorldSpaceInertia(ClosestHandle->GetR() * ClosestHandle->RotationOfMass(), ClosestHandle->InvI());
					const Chaos::FVec3 AngularImpulse = Chaos::FVec3::CrossProduct(WorldLocation - WorldCOM, Impulse);
					ClosestHandle->SetAngularImpulseVelocity(ClosestHandle->AngularImpulseVelocity() + WorldInvI * AngularImpulse, false);
				}
			});
	}
}

void FGeometryCollectionPhysicsProxy::BreakClusters_External(TArray<FGeometryCollectionItemIndex>&& ItemIndices)
{
	check(IsInGameThread());

	if (Chaos::FPhysicsSolver* RBDSolver = GetSolver<Chaos::FPhysicsSolver>())
	{
		RBDSolver->EnqueueCommandImmediate([this, RBDSolver, IndicesToBreakParent = MoveTemp(ItemIndices)]()
		{
			Chaos::FRigidClustering& Clustering = RBDSolver->GetEvolution()->GetRigidClustering();
			for (const FGeometryCollectionItemIndex& ItemIndex : IndicesToBreakParent)
			{
				if (Chaos::FPBDRigidClusteredParticleHandle* ClusteredHandle = FindClusteredParticleHandleByItemIndex_Internal(ItemIndex))
				{
					Clustering.BreakCluster(ClusteredHandle);
				}
			}
		});
	}
}


void FGeometryCollectionPhysicsProxy::BreakActiveClusters_External()
{
	check(IsInGameThread());

	if (Chaos::FPhysicsSolver* RBDSolver = GetSolver<Chaos::FPhysicsSolver>())
	{
		RBDSolver->EnqueueCommandImmediate([this, RBDSolver]()
		{
			Chaos::FRigidClustering& Clustering = RBDSolver->GetEvolution()->GetRigidClustering();
			Clustering.BreakClustersByProxy(this);
		});
	}
}

static void SetParticleAnchored_Internal(Chaos::FPBDRigidsEvolution* Evolution, Chaos::FPBDRigidClusteredParticleHandle* ParticleHandle, bool bAnchored)
{
	if (ParticleHandle)
	{
		ParticleHandle->SetIsAnchored(bAnchored);
		if (!bAnchored)
		{
			if (Evolution && !ParticleHandle->IsDynamic())
			{
				Chaos::FKinematicTarget NoKinematicTarget;
				Evolution->SetParticleObjectState(ParticleHandle, Chaos::EObjectStateType::Dynamic);
				Evolution->SetParticleKinematicTarget(ParticleHandle, NoKinematicTarget);
			}
		}
	}
}
static Chaos::FPBDRigidClusteredParticleHandle* GetTopActiveClusteredParent_Internal(Chaos::FPBDRigidClusteredParticleHandle* ParticleHandle)
{
	while (ParticleHandle)
	{
		Chaos::FPBDRigidClusteredParticleHandle* ParentParticle = ParticleHandle->Parent();
		if (!ParentParticle)
		{
			return ParticleHandle;
		}
		ParticleHandle = ParentParticle;
	}
	return nullptr;
}

void FGeometryCollectionPhysicsProxy::SetAnchoredByIndex_External(int32 Index, bool bAnchored)
{
	check(IsInGameThread());
	if (Chaos::FPhysicsSolver* RBDSolver = GetSolver<Chaos::FPhysicsSolver>())
	{
		RBDSolver->EnqueueCommandImmediate([this, Index, bAnchored, RBDSolver]()
			{
				Chaos::FPBDRigidsEvolution* Evolution = RBDSolver->GetEvolution();
				const int32 ParticleIndex = FromTransformToParticleIndex[Index];
				if (SolverParticleHandles.IsValidIndex(ParticleIndex))
				{
					SetParticleAnchored_Internal(Evolution, SolverParticleHandles[ParticleIndex], bAnchored);
					if (Chaos::FPBDRigidClusteredParticleHandle* TopParentHandle = GetTopActiveClusteredParent_Internal(SolverParticleHandles[ParticleIndex]))
					{
						Chaos::UpdateKinematicProperties(TopParentHandle, Evolution->GetRigidClustering().GetChildrenMap(), *Evolution);
					}
				}		
			});
	}
}

void FGeometryCollectionPhysicsProxy::SetAnchoredByTransformedBox_External(const FBox& Box, const FTransform& Transform, bool bAnchored, int32 MaxLevel)
{
	check(IsInGameThread());
	if (Chaos::FPhysicsSolver* RBDSolver = GetSolver<Chaos::FPhysicsSolver>())
	{
		Chaos::FAABB3 BoundsToCheck(Box.Min, Box.Max);
		Chaos::FRigidTransform3 BoxTransform(Transform);
		RBDSolver->EnqueueCommandImmediate([this, BoundsToCheck, BoxTransform, bAnchored, MaxLevel, RBDSolver]()
			{
				using namespace Chaos;
				TSet<FPBDRigidClusteredParticleHandle*> TopParentHandles;

				const TManagedArrayAccessor<int32> InitialLevelAttribute = PhysicsThreadCollection.GetInitialLevels();
				const int32 MaxLevelToCheck = (InitialLevelAttribute.IsValid()) ? MaxLevel : INDEX_NONE;

				FPBDRigidsEvolution* Evolution = RBDSolver->GetEvolution();
				for (int32 ParticleIndex = 0; ParticleIndex < SolverParticleHandles.Num(); ParticleIndex++)
				{
					if (FClusterHandle* ParticleHandle = SolverParticleHandles[ParticleIndex])
					{
						const FVec3 PositionInBoxSpace = BoxTransform.InverseTransformPosition(ParticleHandle->GetX());
						if (BoundsToCheck.Contains(PositionInBoxSpace))
						{
							// if we have a max level , we make sure to anchor a parent of the right level 
							if (MaxLevelToCheck > INDEX_NONE)
							{
								const int32 ParticleLevel = InitialLevelAttribute.Get()[FromParticleToTransformIndex[ParticleIndex]];
								if (ParticleLevel > MaxLevelToCheck)
								{
									Chaos::FPBDRigidClusteredParticleHandle* ParentParticle = ParticleHandle->Parent();
									int32 ParentParticleLevel = (ParticleLevel - 1);
									// we are above level, so we still contribute to the parent matching the level 
									while (ParentParticle && ParentParticleLevel > MaxLevelToCheck)
									{
										ParentParticle = ParentParticle->Parent();
									}
									// if we still have a parent , this means the level requirement has been found
									// let's use this particle to be anchored
									// if the parent is null, then we don't want to anchor the particle 
									ParticleHandle = ParentParticle;
								}
							}
							if (ParticleHandle)
							{
								SetParticleAnchored_Internal(Evolution, ParticleHandle, bAnchored);
								if (Chaos::FPBDRigidClusteredParticleHandle* TopParentHandle = GetTopActiveClusteredParent_Internal(ParticleHandle))
								{
									TopParentHandles.Add(TopParentHandle);
								}
							}
						}
					}
				}
				// update the top parents state
				for (FPBDRigidClusteredParticleHandle* TopParentHandle : TopParentHandles)
				{
					UpdateKinematicProperties(TopParentHandle, Evolution->GetRigidClustering().GetChildrenMap(), *Evolution);
				}
			});
	}
}
void FGeometryCollectionPhysicsProxy::RemoveAllAnchors_External()
{
	check(IsInGameThread());
	if (Chaos::FPhysicsSolver* RBDSolver = GetSolver<Chaos::FPhysicsSolver>())
	{
		RBDSolver->EnqueueCommandImmediate([this, RBDSolver]()
			{
				Chaos::FPBDRigidsEvolution* Evolution = RBDSolver->GetEvolution();
				for (FClusterHandle* ParticleHandle : SolverParticleHandles)
				{
					SetParticleAnchored_Internal(Evolution, ParticleHandle, false);
					// we do not need to update the kinematic state since everything is now dynamic 
					// todo(chaos) what about cluster groups ? 
				}
			});
	}
}

template <typename TAction>
static void ApplyToChildrenAtPointWithRadiusAndPropagation_Internal(
	Chaos::FRigidClustering& Clustering,
	Chaos::FPBDRigidClusteredParticleHandle& ClusteredHandle,
	const FVector& WorldLocation,
	float Radius,
	int32 PropagationDepth,
	float PropagationFactor,
	TAction Action)
{
	const bool bIsCluster = (ClusteredHandle.ClusterIds().NumChildren > 0);
	if (bIsCluster)
	{
		TArray<Chaos::FPBDRigidParticleHandle*> ChildrenHandles = Clustering.FindChildrenWithinRadius(&ClusteredHandle, WorldLocation, Radius, true /* always return closest */ );
		if (ChildrenHandles.Num())
		{
			TArray<Chaos::FPBDRigidParticleHandle*> ProcessedHandles;
			Chaos::FRealSingle PropagationMultiplier = 1.0f;

			int32 CurrentPropagationDepth = PropagationDepth;
			while (CurrentPropagationDepth >= 0)
			{
				// apply to current
				for (Chaos::FPBDRigidParticleHandle* ChildHandle: ChildrenHandles)
				{
					if (Chaos::FPBDRigidClusteredParticleHandle* ClusteredChildHandle = ChildHandle->CastToClustered())
					{
						Action(ClusteredChildHandle, PropagationMultiplier);
//						ClusteredChildHandle->SetExternalStrain(FMath::Max(ClusteredChildHandle->GetExternalStrain(), StrainValue * PropagationMultiplier));
					}
				}

				// find handles to propagate to
				if (CurrentPropagationDepth > 0)
				{
					// only propagate to newly added ( avoids allocating a new temporary array for next level handles )
					// move the entire ChildrenHandle array to the processed ones so that it also resets it for us
					const int32 StartIndex = ProcessedHandles.Num(); 
					ProcessedHandles.Append(MoveTemp(ChildrenHandles));
					
					for (int32 ChildIndex = StartIndex; ChildIndex < ProcessedHandles.Num(); ++ChildIndex)
					{
						if (Chaos::FPBDRigidClusteredParticleHandle* ClusteredChildHandle = ProcessedHandles[ChildIndex]->CastToClustered())
						{
							for (const Chaos::TConnectivityEdge<Chaos::FReal>& Edge:ClusteredChildHandle->ConnectivityEdges())
							{
								if (Edge.Sibling && !ChildrenHandles.Contains(Edge.Sibling))
								{
									ChildrenHandles.Add(Edge.Sibling);
								}
							}
						}
					}
				}
				
				PropagationMultiplier *= PropagationFactor;
				--CurrentPropagationDepth;
			}
		}
	}
	else
	{
		Action(&ClusteredHandle, 1.0);
	}
}


void FGeometryCollectionPhysicsProxy::ApplyExternalStrain_External(FGeometryCollectionItemIndex ItemIndex, const FVector& WorldLocation, float Radius, int32 PropagationDepth, float PropagationFactor, float StrainValue)
{
	check(IsInGameThread());

	if (Chaos::FPhysicsSolver* RBDSolver = GetSolver<Chaos::FPhysicsSolver>())
	{
		RBDSolver->EnqueueCommandImmediate([this, Radius, StrainValue, PropagationDepth, PropagationFactor, RBDSolver, WorldLocation, ItemIndex]()
		{
			if (Chaos::FPBDRigidClusteredParticleHandle* ClusteredHandle = FindClusteredParticleHandleByItemIndex_Internal(ItemIndex))
			{
				Chaos::FRigidClustering& RigidClustering = RBDSolver->GetEvolution()->GetRigidClustering();
				ApplyToChildrenAtPointWithRadiusAndPropagation_Internal(
					RBDSolver->GetEvolution()->GetRigidClustering(), *ClusteredHandle,
					WorldLocation, Radius, PropagationDepth, PropagationFactor,
					[StrainValue, &RigidClustering](Chaos::FPBDRigidClusteredParticleHandle* ClusteredChildHandle, Chaos::FRealSingle PropagationMultiplier)
					{
						RigidClustering.SetExternalStrain(ClusteredChildHandle, FMath::Max(ClusteredChildHandle->GetExternalStrain(), StrainValue * PropagationMultiplier));
					});
			}
		});
	}
}

void FGeometryCollectionPhysicsProxy::ApplyInternalStrain_External(FGeometryCollectionItemIndex ItemIndex, const FVector& WorldLocation, float Radius, int32 PropagationDepth, float PropagationFactor, float StrainValue)
{
	check(IsInGameThread());

	if (Chaos::FPhysicsSolver* RBDSolver = GetSolver<Chaos::FPhysicsSolver>())
	{
		RBDSolver->EnqueueCommandImmediate([this, Radius, StrainValue, PropagationDepth, PropagationFactor, RBDSolver, WorldLocation, ItemIndex]()
		{
			if (Chaos::FPBDRigidClusteredParticleHandle* ClusteredHandle = FindClusteredParticleHandleByItemIndex_Internal(ItemIndex))
			{
				ApplyToChildrenAtPointWithRadiusAndPropagation_Internal(
					RBDSolver->GetEvolution()->GetRigidClustering(), *ClusteredHandle,
					WorldLocation, Radius, PropagationDepth, PropagationFactor,
					[StrainValue, RBDSolver](Chaos::FPBDRigidClusteredParticleHandle* ClusteredChildHandle, Chaos::FRealSingle PropagationMultiplier)
					{
						const Chaos::FRealSingle NewInternalStrain = ClusteredChildHandle->GetInternalStrains() - (StrainValue * PropagationMultiplier);
						RBDSolver->GetEvolution()->GetRigidClustering().SetInternalStrain(ClusteredChildHandle, FMath::Max(0, NewInternalStrain));
					});
			}
		});
	}
}

template <typename TAction>
static void ApplyToBreakingChildren_Internal(Chaos::FRigidClustering& Clustering, Chaos::FPBDRigidClusteredParticleHandle& ClusteredHandle, TAction Action)
{
	const bool bIsCluster = (ClusteredHandle.ClusterIds().NumChildren > 0);
	const bool bHasClusterUnionParent = Clustering.GetClusterUnionManager().IsClusterUnionParticle(ClusteredHandle.Parent());
	if (!ClusteredHandle.Disabled() || bHasClusterUnionParent)
	{
		if (bIsCluster)
		{
			Chaos::FRigidClustering::FClusterMap& ChildrenMap = Clustering.GetChildrenMap();
			if (const TArray<Chaos::FPBDRigidParticleHandle*>* ChildrenHandles = ChildrenMap.Find(&ClusteredHandle))
			{
				for (Chaos::FPBDRigidParticleHandle* ChildHandle : *ChildrenHandles)
				{
					if (Chaos::FPBDRigidClusteredParticleHandle* ClusteredChildHandle = ChildHandle->CastToClustered())
					{
						// todo(chaos) : this does not account for the various damage models, we should eventually call an evaluate function to avoid replicating logic from the clustering code
						// also we cannot account for collision impulses because they are set after the physics callbacks are evaluated
						if (ClusteredChildHandle->GetExternalStrain() >= ClusteredChildHandle->GetInternalStrains())
						{
							Action(ClusteredChildHandle);
						}
					}
				}
			}
		}
		else
		{
			// leaf node , let's just apply to it directly
			if (ClusteredHandle.GetExternalStrain() >= ClusteredHandle.GetInternalStrains())
			{
				Action(&ClusteredHandle);
			}
		}
	}
}

void FGeometryCollectionPhysicsProxy::ApplyBreakingLinearVelocity_External(FGeometryCollectionItemIndex ItemIndex, const FVector& LinearVelocity)
{
	check(IsInGameThread());

	if (Chaos::FPhysicsSolver* RBDSolver = GetSolver<Chaos::FPhysicsSolver>())
	{
		RBDSolver->EnqueueCommandImmediate([this, RBDSolver, ItemIndex, LinearVelocity]()
		{
			if (Chaos::FPBDRigidClusteredParticleHandle* ClusteredHandle = FindClusteredParticleHandleByItemIndex_Internal(ItemIndex))
			{
				ApplyToBreakingChildren_Internal(RBDSolver->GetEvolution()->GetRigidClustering(), *ClusteredHandle,
					[LinearVelocity, RBDSolver](Chaos::FPBDRigidClusteredParticleHandle* ClusteredChildHandle)
					{
						ClusteredChildHandle->SetV(ClusteredChildHandle->GetV() + LinearVelocity);
						if (!ClusteredChildHandle->GetV().IsNearlyZero() && !ClusteredChildHandle->Disabled())
						{
							RBDSolver->GetEvolution()->WakeParticle(ClusteredChildHandle);
						}
					});
			}
		});
	}
}

void FGeometryCollectionPhysicsProxy::ApplyBreakingAngularVelocity_External(FGeometryCollectionItemIndex ItemIndex, const FVector& AngularVelocity)
{
	check(IsInGameThread());

	if (Chaos::FPhysicsSolver* RBDSolver = GetSolver<Chaos::FPhysicsSolver>())
	{
		RBDSolver->EnqueueCommandImmediate([this, RBDSolver, ItemIndex, AngularVelocity]()
		{
			if (Chaos::FPBDRigidClusteredParticleHandle* ClusteredHandle = FindClusteredParticleHandleByItemIndex_Internal(ItemIndex))
			{
				ApplyToBreakingChildren_Internal(RBDSolver->GetEvolution()->GetRigidClustering(), *ClusteredHandle,
					[AngularVelocity, RBDSolver](Chaos::FPBDRigidClusteredParticleHandle* ClusteredChildHandle)
					{
						ClusteredChildHandle->SetW(ClusteredChildHandle->GetW() + AngularVelocity);
						if (!ClusteredChildHandle->GetW().IsNearlyZero() && !ClusteredChildHandle->Disabled())
						{
							RBDSolver->GetEvolution()->WakeParticle(ClusteredChildHandle);
						}
					});
			}
		});
	}
}

void FGeometryCollectionPhysicsProxy::ApplyLinearVelocity_External(FGeometryCollectionItemIndex ItemIndex, const FVector& LinearVelocity)
{
	check(IsInGameThread());

	if (Chaos::FPhysicsSolver* RBDSolver = GetSolver<Chaos::FPhysicsSolver>())
	{
		RBDSolver->EnqueueCommandImmediate([this, ItemIndex, LinearVelocity, RBDSolver]()
		{
			if (Chaos::FPBDRigidClusteredParticleHandle* ClusteredHandle = FindClusteredParticleHandleByItemIndex_Internal(ItemIndex))
			{
				ClusteredHandle->SetV(ClusteredHandle->GetV() + LinearVelocity);
				if (!ClusteredHandle->GetV().IsNearlyZero() && !ClusteredHandle->Disabled())
				{
					RBDSolver->GetEvolution()->WakeParticle(ClusteredHandle);
				}
			}
		});
	}
}

void FGeometryCollectionPhysicsProxy::ApplyAngularVelocity_External(FGeometryCollectionItemIndex ItemIndex, const FVector& AngularVelocity)
{
	check(IsInGameThread());

	if (Chaos::FPhysicsSolver* RBDSolver = GetSolver<Chaos::FPhysicsSolver>())
	{
		RBDSolver->EnqueueCommandImmediate([this, ItemIndex, AngularVelocity, RBDSolver]()
		{
			if (Chaos::FPBDRigidClusteredParticleHandle* ClusteredHandle = FindClusteredParticleHandleByItemIndex_Internal(ItemIndex))
			{
				ClusteredHandle->SetW(ClusteredHandle->GetW() + AngularVelocity);
				if (!ClusteredHandle->GetW().IsNearlyZero() && !ClusteredHandle->Disabled())
				{
					RBDSolver->GetEvolution()->WakeParticle(ClusteredHandle);
				}
			}
		});
	}
}

void FGeometryCollectionPhysicsProxy::SetProxyDirty_External()
{
	check(IsInGameThread());
	if (Chaos::FPhysicsSolver* RBDSolver = GetSolver<Chaos::FPhysicsSolver>())
	{
		RBDSolver->AddDirtyProxy(this);
	}
}

void FGeometryCollectionPhysicsProxy::SetEnableDamageFromCollision_External(bool bEnable)
{
	ExecuteOnPhysicsThread(*this,
		[this, bEnable]()
		{
			Parameters.bEnableStrainOnCollision = bEnable;
		});
}

void FGeometryCollectionPhysicsProxy::SetNotifyBreakings_External(bool bNotify)
{
	ExecuteOnPhysicsThread(*this,
		[this, bNotify]()
		{
			Parameters.bGenerateBreakingData = bNotify;
		});
}

void FGeometryCollectionPhysicsProxy::SetNotifyTrailings_External(bool bNotify)
{
	ExecuteOnPhysicsThread(*this,
		[this, bNotify]()
		{
			Parameters.bGenerateTrailingData = bNotify;
		});
}

void FGeometryCollectionPhysicsProxy::SetNotifyRemovals_External(bool bNotify)
{
	// nothing to do : there's no removal flag to set on the proxy game thread or physics thread side
	// todo(chaos) we shoudl probably have one as we may add to the removal array regardless of what the user sets on the component side 
}

void FGeometryCollectionPhysicsProxy::SetNotifyCrumblings_External(bool bNotify, bool bIncludeChildren)
{
	ExecuteOnPhysicsThread(*this,
		[this, bNotify, bIncludeChildren]()
		{
			Parameters.bGenerateCrumblingData = bNotify;
			Parameters.bGenerateCrumblingChildrenData = bIncludeChildren;

		});
}

void FGeometryCollectionPhysicsProxy::SetNotifyGlobalBreakings_External(bool bNotify)
{
	ExecuteOnPhysicsThread(*this,
		[this, bNotify]()
		{
			Parameters.bGenerateGlobalBreakingData = bNotify;
			if (bNotify)
			{
				if (Chaos::FPBDRigidsSolver* Solver = GetSolver<Chaos::FPBDRigidsSolver>())
				{
					Solver->SetGenerateBreakingData(true);
				}
			}
		});
}

void FGeometryCollectionPhysicsProxy::SetNotifyGlobalRemovals_External(bool bNotify)
{
	// nothing to do : there's no removal flag to set on the proxy game thread or physics thread side
	// todo(chaos) we shoudl probably have one as we may add to the removal array regardless of what the user sets on the component side 
}

void FGeometryCollectionPhysicsProxy::SetNotifyGlobalCrumblings_External(bool bNotify, bool bIncludeChildren)
{
	ExecuteOnPhysicsThread(*this,
		[this, bNotify, bIncludeChildren]()
		{
			Parameters.bGenerateGlobalCrumblingData = bNotify;
			Parameters.bGenerateGlobalCrumblingChildrenData = bIncludeChildren;

		});
}

void FGeometryCollectionPhysicsProxy::SetNotifyGlobalTrailings_External(bool bNotify)
{
	ExecuteOnPhysicsThread(*this,
		[this, bNotify]()
		{
			Parameters.bGenerateGlobalTrailingData = bNotify;
			if (bNotify)
			{
				if (Chaos::FPBDRigidsSolver* Solver = GetSolver<Chaos::FPBDRigidsSolver>())
				{
					Solver->SetGenerateTrailingData(true);
				}
			}
		});
}

int32 FGeometryCollectionPhysicsProxy::CalculateHierarchyLevel(const FGeometryDynamicCollection& DynamicCollection, int32 TransformIndex)
{
	int32 Level = 0;
	while ((TransformIndex = DynamicCollection.GetParent(TransformIndex)) != INDEX_NONE)
	{
		Level++;
	}
	return Level;
}

TBitArray<> FGeometryCollectionPhysicsProxy::CalculateClustersToCreateFromChildren(const FGeometryDynamicCollection& DynamicCollection, int32 NumTransforms)
{
	// Generate a bitmask of all the particles that are clusters without any collision geometry, that have children with collision geometry
	// This is a workaround following the fix to not create particles that have no geometry. If bGeometryCollectionAlwaysGenerateGTCollisionForClusters
	// is set, we will be auto-generating cluster geometry from their children (if they have geometry)
	// @todo(chaos): this dupes functionality in FGeometryCollectionPhysicsProxy::Initialize
	TBitArray<> ClustersToGenerate;

	if (bGeometryCollectionAlwaysGenerateGTCollisionForClusters)
	{
		ClustersToGenerate.Init(false, NumTransforms);

		const TManagedArray<Chaos::FImplicitObjectPtr>* Implicits = DynamicCollection.FindAttribute<Chaos::FImplicitObjectPtr>(FGeometryDynamicCollection::ImplicitsAttribute, FTransformCollection::TransformGroup);

		// All the leaf objects
		TArray<int32> ChildrenToCheckForParentFix;
		for (int32 Index = 0; Index < NumTransforms; ++Index)
		{
			if (!DynamicCollection.HasChildren(Index))
			{
				ChildrenToCheckForParentFix.Add(Index);
			}
		}

		TSet<int32> ParentToPotentiallyFix;
		while (ChildrenToCheckForParentFix.Num())
		{
			// step 1 : find parents
			for (const int32 ChildIndex : ChildrenToCheckForParentFix)
			{
				if (int32 ParentIndex = DynamicCollection.GetParent(ChildIndex); ParentIndex != INDEX_NONE)
				{
					ParentToPotentiallyFix.Add(ParentIndex);
				}
			}

			// step 2: test the parent for having children with geometry
			for (const int32 ParentToFixIndex : ParentToPotentiallyFix)
			{
				const bool bParentHasCollision = Implicits && (((*Implicits)[ParentToFixIndex] != nullptr) || ClustersToGenerate[ParentToFixIndex]);

				if (!bParentHasCollision)
				{
					// let's make sure all our children have an implicit defined, otherwise, postpone to next iteration 
					bool bAllChildrenHaveCollision = true;
					DynamicCollection.IterateThroughChildren(ParentToFixIndex, [&](int32 ChildIndex)
					{
						// defer if any of the children is a cluster with no collision yet generated 
						const bool bChildHasCollision = Implicits && (((*Implicits)[ChildIndex] != nullptr) || ClustersToGenerate[ChildIndex]);
						if (!bChildHasCollision && DynamicCollection.HasChildren(ChildIndex))
						{
							bAllChildrenHaveCollision = false;
							return false;
						}
						return true;
					});

					ClustersToGenerate[ParentToFixIndex] = bAllChildrenHaveCollision;
				}
			}

			// step 3 : make the parent the new child to go up the hierarchy and continue the fixing
			ChildrenToCheckForParentFix = ParentToPotentiallyFix.Array();
			ParentToPotentiallyFix.Reset();
		}
	}

	return ClustersToGenerate;
}

int32 FGeometryCollectionPhysicsProxy::CalculateEffectiveParticles(const FGeometryDynamicCollection& DynamicCollection, int32 NumTransform, int32 InMaxSimulatedLevel, bool bEnableClustering, const UObject* Owner, TBitArray<>& EffectiveParticles)
{
	TBitArray<> ClustersUsingChildGeometry = CalculateClustersToCreateFromChildren(DynamicCollection, NumTransform);

	int32 NumMissingGeometry = 0;
	int32 NumEffectiveParticlesFound = 0;
	EffectiveParticles.Init(false, NumTransform);
	const int32 MaxSimulatedLevel = FMath::Min(GlobalMaxSimulatedLevel, InMaxSimulatedLevel);
	const TManagedArray<Chaos::FImplicitObjectPtr>* Implicits = DynamicCollection.FindAttribute<Chaos::FImplicitObjectPtr>(FGeometryDynamicCollection::ImplicitsAttribute, FTransformCollection::TransformGroup);
	TManagedArrayAccessor<int32> Levels = DynamicCollection.GetInitialLevels();

	for (int32 TransformIndex = 0; TransformIndex < NumTransform; ++TransformIndex)
	{
		const int32 Level = Levels[TransformIndex];
		if (Level <= MaxSimulatedLevel || !bEnableClustering)
		{
			const bool bIsClusterUsingChildGeometry = (ClustersUsingChildGeometry.Num() > 0) && ClustersUsingChildGeometry[TransformIndex];
			const bool bHasGeometry = Implicits && (*Implicits)[TransformIndex].IsValid();

			if (bHasGeometry || bIsClusterUsingChildGeometry)
			{
				EffectiveParticles[TransformIndex] = true;
				NumEffectiveParticlesFound++;
			}
			else
			{
				// This will happen if some particles are being culled (e.g., via data flow) but the MaxSimulatedLevel is set to a level that includes them.
				UE_LOGF(LogChaos, Verbose, "GeometryCollection Transform %d has no geometry", TransformIndex);
				NumMissingGeometry++;
			}
		}
	}

	if (NumMissingGeometry > 0)
	{
		if (Owner != nullptr)
		{
			UE_LOGF(LogChaos, Verbose, "Geometry collection %ls tried to create %d particles with no geometry", *Owner->GetFullName(), NumMissingGeometry);
		}
		else
		{
			UE_LOGF(LogChaos, Verbose, "Geometry collection tried to create %d particles with no geometry", NumMissingGeometry);
		}
	}

	return NumEffectiveParticlesFound;
}


int32 FGeometryCollectionPhysicsProxy::CalculateAndSetLevel(int32 TransformGroupIndex, const TManagedArray<int32>& Parent, TManagedArray<int32>& Levels)
{
	// Count levels up to root or first parent that has level initialized
	int32 LevelsFromRoot = 0;

	// Parents up tree from current index that need level initialization, ordered depth first
	TArray<int32, TInlineAllocator<8>> ParentStack;

	int32 ParentTransformGroupIndex = Parent[TransformGroupIndex];
	while (ParentTransformGroupIndex != INDEX_NONE)
	{
		LevelsFromRoot++;

		int32 ParentLevel = Levels[ParentTransformGroupIndex];
		if (ParentLevel > 0)
		{
			// If parent level is not 0 it has already been computed, 
			// can early out and not search for true root.
			Levels[TransformGroupIndex] = ParentLevel + LevelsFromRoot;
			break;
		}

		// save this parent so we can set it's level once we determine depth
		ParentStack.Push(ParentTransformGroupIndex);

		ParentTransformGroupIndex = Parent[ParentTransformGroupIndex];
	}

	if (LevelsFromRoot > 0) // Not at root, update levels of current particle and uninitialized parents
	{
		if (Levels[TransformGroupIndex] == 0)
		{
			// If we are uninitialized, we did not early out after finding an initialized parent up the tree, and LevelsFromRoot is new level.
			Levels[TransformGroupIndex] = LevelsFromRoot;
		}

		// Initialize newly discovered parent levels
		for (int32 Idx = 0; Idx < ParentStack.Num(); ++Idx)
		{
			int32 ParentTransformGroupIdx = ParentStack[Idx];

			// Level of parent is leaf level minus parent's distance from leaf minus 1.
			Levels[ParentTransformGroupIdx] = Levels[TransformGroupIndex] - Idx - 1;
		}
	}

	return Levels[TransformGroupIndex];
}

void FGeometryCollectionPhysicsProxy::OnUnregisteredFromSolver()
{
	IsObjectDeleting = true;
}

void FGeometryCollectionPhysicsProxy::OnRemoveFromSolver(Chaos::FPBDRigidsSolver *RBDSolver)
{
	Chaos::FPBDRigidsEvolutionGBF* Evolution = RBDSolver->GetEvolution();

	TSet< FClusterHandle* > ClustersToRebuild;
	for (int i = 0; i < SolverParticleHandles.Num(); i++)
	{
		if (FClusterHandle* Handle = SolverParticleHandles[i])
		{
			if (FClusterHandle* ParentCluster = Evolution->GetRigidClustering().DestroyClusterParticle(Handle))
			{
				// We do not want to do anything relating to any cluster unions that might own this geometry collection.
				IPhysicsProxyBase* ParentClusterProxy = ParentCluster->PhysicsProxy();
				if (ParentCluster->InternalCluster() && ParentClusterProxy == this)
				{
					ClustersToRebuild.Add(ParentCluster);
				}
			}
		}
	}

	Evolution->GetRigidClustering().GetClusterUnionManager().HandleDeferredClusterUnionUpdateProperties();

	for (int i = 0; i < SolverParticleHandles.Num(); i++)
	{
		if (FClusterHandle* Handle = SolverParticleHandles[i])
		{
			Chaos::FUniqueIdx UniqueIdx = Handle->UniqueIdx();
			Evolution->DestroyParticle(Handle);
			Evolution->ReleaseUniqueIdx(UniqueIdx);
			SolverParticleHandles[i] = nullptr;
		}
	}

	for (FClusterHandle* Cluster : ClustersToRebuild)
	{
		ensure(Cluster->InternalCluster());
		if (ensure(Evolution->GetRigidClustering().GetChildrenMap().Contains(Cluster)))
		{
			// copy cluster state for recreation
			int32 ClusterGroupIndex = Cluster->ClusterGroupIndex();
			TArray<FParticleHandle*> Children = Evolution->GetRigidClustering().GetChildrenMap()[Cluster];

			// destroy the invalid cluster
			FClusterHandle* NullHandle = Evolution->GetRigidClustering().DestroyClusterParticle(Cluster);
			ensure(NullHandle == nullptr);

			// create a new cluster if needed
			if (Children.Num())
			{
				if (FClusterHandle* NewParticle = Evolution->GetRigidClustering().CreateClusterParticle(ClusterGroupIndex, MoveTemp(Children), Chaos::FClusterCreationParameters(), Chaos::FImplicitObjectPtr(nullptr)))
				{
					NewParticle->SetInternalCluster(true);
				}
			}
		}		
	}
}

void FGeometryCollectionPhysicsProxy::OnRemoveFromScene()
{
#if TODO_REIMPLEMENT_GET_RIGID_PARTICLES
	// #BG TODO This isn't great - we currently cannot handle things being removed from the solver.
	// need to refactor how we handle this and actually remove the particles instead of just constantly
	// growing the array. Currently everything is just tracked by index though so the solver will have
	// to notify all the proxies that a chunk of data was removed - or use a sparse array (undesireable)
	Chaos::FPhysicsSolver::FParticlesType& Particles = GetSolver<FSolver>()->GetRigidParticles();

	// #BG TODO Special case here because right now we reset/realloc the evolution per geom component
	// in endplay which clears this out. That needs to not happen and be based on world shutdown
	if(Particles.Size() == 0)
	{
		return;
	}

	const int32 Begin = BaseParticleIndex;
	const int32 Count = NumTransforms;

	if (ensure((int32)Particles.Size() > 0 && (Begin + Count) <= (int32)Particles.Size()))
	{
		for (int32 ParticleIndex = 0; ParticleIndex < Count; ++ParticleIndex)
		{
			GetSolver<FSolver>()->GetEvolution()->DisableParticle(Begin + ParticleIndex);
			GetSolver<FSolver>()->GetRigidClustering().GetTopLevelClusterParents().Remove(Begin + ParticleIndex);
		}
	}
#endif
}

void FGeometryCollectionPhysicsProxy::SyncBeforeDestroy()
{
	// THIS HAPPENS ON THE PHYSICS THREAD.

	if (Chaos::FPhysicsSolver* RigidSolver = GetSolver<Chaos::FPhysicsSolver>())
	{
		Chaos::FRigidClustering& RigidClustering = RigidSolver->GetEvolution()->GetRigidClustering();
		Chaos::FClusterUnionManager& ClusterUnionManager = RigidClustering.GetClusterUnionManager();
		for (Chaos::FPBDRigidClusteredParticleHandle* Handle : SolverParticleHandles)
		{
			if (Handle)
			{
				// If this particle is in an internal cluster that belongs to the geometry collection, the internal cluster needs to be disabled too.
				if (Chaos::FPBDRigidClusteredParticleHandle* Parent = Handle->Parent())
				{
					if (Parent->PhysicsProxy() == this && !Parent->Disabled())
					{
						// We should be able to just disabled it - an internal cluster should never be considered in a cluster union.
						RigidClustering.DisableCluster(Parent);
					}
				}
		
				// It should be safe to defer here without an immediate call to HandleDeferredClusterUnionUpdateProperties.
				// This change doesn't really have to go through until FClusterUnionManager::FlushPendingOperations which happens
				// prior to trying to advance the frame.
				ClusterUnionManager.HandleRemoveOperationWithClusterLookup( { Handle }, Chaos::EClusterUnionOperationTiming::Defer);
		
				// Need to use FRigidClustering::DisableCluster instead of the evolution's to also handle the fact that the GC particle could be in the top level strained sets
				// which would make it get considered in the breaking model which is undesirable.
				RigidClustering.DisableCluster(Handle);
		
				// This ensures that this particle won't have any intercluster edges on it. This way no connectivity operations will try to add it into a cluster union.
				RigidClustering.RemoveNodeConnections(Handle);
			}
		}
	}
}

void FGeometryCollectionPhysicsProxy::BufferGameState() 
{
	//
	// There is currently no per advance updates to the GeometryCollection
	//
}


void FGeometryCollectionPhysicsProxy::UpdateWorldTransformCaches_External(const FTransform& WorldTransform)
{
	if (WorldTransform.Equals(WorldTransform_External))
	{
		return;
	}

	bIsGameThreadWorldTransformDirty = true;
	PreviousWorldTransform_External = WorldTransform_External;
	WorldTransform_External = WorldTransform;

	// Mirror to the PT-side cache only; do not run the kinematic / cluster-union / rescale
	// side effects that SetWorldTransform_Internal performs. PrevWorldTransform stays under
	// end-of-tick snapshot ownership (see BufferPhysicsResults_Internal).
	ExecuteOnPhysicsThread(*this,
		[this, WorldTransform]()
		{
			Parameters.WorldTransform = WorldTransform;
		});
}

void FGeometryCollectionPhysicsProxy::SetWorldTransform_External(const FTransform& WorldTransform)
{
	// todo : change to compare with previous value
	const bool bHasTransformChanged = !WorldTransform.Equals(WorldTransform_External);
	if (bHasTransformChanged)
	{
		bIsGameThreadWorldTransformDirty = bHasTransformChanged;
		PreviousWorldTransform_External = WorldTransform_External;
		WorldTransform_External = WorldTransform;

		// update GT particle if needed - experimental
		if (bUpdateGTParticlesOnWorldTransformUpdate)
		{
			const TManagedArray<bool>* AnimationsActive = GameThreadCollection.GetAnimateTransformAttribute();

			const int32 NumParticles = GTParticles.Num();
			for (int32 ParticleIndex = 0; ParticleIndex < NumParticles; ParticleIndex++)
			{
				if (const TUniquePtr<FParticle>& GTParticlePtr = GTParticles[ParticleIndex])
				{
					const int32 TransformIndex = FromParticleToTransformIndex[ParticleIndex];
					const bool bAnimatingWhileDisabled = AnimationsActive ? (*AnimationsActive)[TransformIndex] : false;
					const bool bActive = GameThreadCollection.Active[TransformIndex];
					if (bActive || bAnimatingWhileDisabled)
					{
						const FTransform PreviousParticleTransform(GTParticlePtr->R(), GTParticlePtr->X());
						const FTransform RelativeParticleTransform = PreviousParticleTransform.GetRelativeTransform(PreviousWorldTransform_External);
						const FTransform NewParticleTransform = RelativeParticleTransform * WorldTransform_External;
						GTParticlePtr->SetX(NewParticleTransform.GetTranslation());
						GTParticlePtr->SetR(NewParticleTransform.GetRotation());
					}
				}
			}
		}

		// Because SetWorldTransform_Internal is running on the physics thread we need to capture this parameter 
		// (it is transient and may changed after calling this function)
		const bool bSkipChildToParentUpdate = GetSkipChildToParentUpdateWhenInClusterUnion();
		ExecuteOnPhysicsThread(*this,
			[this, WorldTransform, bSkipChildToParentUpdate]()
			{	
				SetWorldTransform_Internal(WorldTransform, bSkipChildToParentUpdate);
			});
	}
}

void FGeometryCollectionPhysicsProxy::ScaleClusterGeometry_Internal(const FVector& WorldScale)
{
	Chaos::FPhysicsSolver* RigidSolver = GetSolver<Chaos::FPhysicsSolver>();

	for (int32 ParticleIndex = 0; ParticleIndex < NumEffectiveParticles; ++ParticleIndex)
	{
		if (Chaos::FPBDRigidClusteredParticleHandle* ParticleHandle = SolverParticleHandles[ParticleIndex])
		{
			if (RigidSolver != nullptr)
			{
				// We are about to change the geometry, so must reset any systems that depend on it (e.g., collisions)
				RigidSolver->GetEvolution()->InvalidateParticle(ParticleHandle);
			}

			// Scale the geometry if necessary
			BuildScaledGeometry(ParticleHandle, ParticleHandle->GetGeometry(), WorldScale);

			// Update acceleration structure, bounds and collision flags
			UpdateCollisionFlags(ParticleHandle, false);
		}
	}
}

void FGeometryCollectionPhysicsProxy::SetWorldTransform_Internal(const FTransform& InWorldTransform, bool bInSkipChildToParentUpdateWhenInClusterUnion)
{
	using namespace Chaos;

	Parameters.WorldTransform = InWorldTransform;

	TSet<FClusterHandle*> ProcessedInternalClusters;
	const FTransform& ActorToWorld = Parameters.WorldTransform;

	check(NumTransforms == PhysicsThreadCollection.NumElements(FGeometryCollection::TransformGroup));
	if (Chaos::FPhysicsSolver* RigidSolver = GetSolver<Chaos::FPhysicsSolver>())
	{
		FClusterUnionManager& ClusterUnionManager = RigidSolver->GetEvolution()->GetRigidClustering().GetClusterUnionManager();

		// Assume all particles are in the same cluster union.
		FClusterUnionIndex ClusterUnionIndex = INDEX_NONE;
		TArray<FPBDRigidParticleHandle*> DeferredClusterUnionParticleUpdates;
		TArray<FTransform> DeferredClusterUnionChildToParentUpdates;

		const TManagedArray<FTransform>& MassToLocal = Parameters.RestCollectionShared->GetAttribute<FTransform>(MassToLocalAttributeName, FTransformCollection::TransformGroup);

		for (int32 ParticleIndex = 0; ParticleIndex < NumEffectiveParticles; ++ParticleIndex)
		{
			if (Chaos::FPBDRigidClusteredParticleHandle* Handle = SolverParticleHandles[ParticleIndex])
			{
				FClusterHandle* KinematicRootHandle = nullptr;
				FClusterHandle* ParentHandle = Handle->Parent();

				if (!Handle->Disabled() && Handle->ObjectState() == Chaos::EObjectStateType::Kinematic)
				{
					KinematicRootHandle = Handle;
				}
				else
				{
					// is there a internal parent as a kinematic root?
					if (ParentHandle && ParentHandle->InternalCluster() && !ParentHandle->Disabled() && ParentHandle->ObjectState() == Chaos::EObjectStateType::Kinematic && ParentHandle->PhysicsProxy() == this)
					{
						if (!ProcessedInternalClusters.Contains(ParentHandle))
						{
							ProcessedInternalClusters.Add(ParentHandle);
							KinematicRootHandle = ParentHandle;
						}
					}
				}

				if (KinematicRootHandle)
				{
					const FTransform RootWorldTransform(KinematicRootHandle->GetR(), KinematicRootHandle->GetX());
					const FTransform RootRelativeTransform = RootWorldTransform.GetRelativeTransform(Parameters.PrevWorldTransform);
					const FTransform WorldTransform = RootRelativeTransform * ActorToWorld;

					SetClusteredParticleKinematicTarget_Internal(KinematicRootHandle, WorldTransform);
				}
				else if (ParentHandle && !ParentHandle->IsDynamic())
				{
					if (ClusterUnionIndex == INDEX_NONE)
					{
						ClusterUnionIndex = ClusterUnionManager.FindClusterUnionIndexFromParticle(Handle);
					}

					if (ClusterUnionIndex != INDEX_NONE)
					{
						const int32 TransformGroupIndex = FromParticleToTransformIndex[ParticleIndex];
						const FTransform ParentWorldTransform{ ParentHandle->GetR(), ParentHandle->GetX() };
						const FTransform NewWorldTransform = MassToLocal[TransformGroupIndex] * FTransform(PhysicsThreadCollection.GetTransform(TransformGroupIndex)) * Parameters.WorldTransform;
						const FTransform RelativeTransform = NewWorldTransform.GetRelativeTransform(ParentWorldTransform);

						DeferredClusterUnionParticleUpdates.Add(Handle);
						DeferredClusterUnionChildToParentUpdates.Add(RelativeTransform);

						// Make sure the particle is mark dirty to make sure its proxy will properly update the transforms
						RigidSolver->GetEvolution()->GetParticles().MarkTransientDirtyParticle(Handle);
					}
				}
			}
		}

		// Should be safe enough to access on the PT.
		const bool bIsAuthority = GetReplicationMode() == EReplicationMode::Server;
		// This should only happen on the server otherwise the client may override the replicated child to parent before it's even set.
		if (bIsAuthority && ClusterUnionIndex != INDEX_NONE && !DeferredClusterUnionParticleUpdates.IsEmpty() && !DeferredClusterUnionChildToParentUpdates.IsEmpty())
		{
			if (!bInSkipChildToParentUpdateWhenInClusterUnion)
			{
				ClusterUnionManager.UpdateClusterUnionParticlesChildToParent(ClusterUnionIndex, DeferredClusterUnionParticleUpdates, DeferredClusterUnionChildToParentUpdates, false);
			}
		}
	}
	if (bGeometryCollectionScaleClusterGeometry)
	{
		const FVector ScaleRatio = Parameters.WorldTransform.GetScale3D() / Parameters.PrevWorldTransform.GetScale3D();
		if (!ScaleRatio.Equals(FVector::OneVector))
		{
			ScaleClusterGeometry_Internal(ScaleRatio);
		}
	}
}

DECLARE_CYCLE_STAT(TEXT("FGeometryCollectionPhysicsProxy::SetUseStaticMeshCollisionForTraces_External"), STAT_SetUseStaticMeshCollisionForTraces_External, STATGROUP_Chaos);
void FGeometryCollectionPhysicsProxy::SetUseStaticMeshCollisionForTraces_External(bool bInUseStaticMeshCollisionForTraces)
{
	SCOPE_CYCLE_COUNTER(STAT_SetUseStaticMeshCollisionForTraces_External);

	check(IsInGameThread());

	// Expensive, so only do it when we need to
	const bool bUseSMCollisionForTraces = ((bInUseStaticMeshCollisionForTraces && ForceOverrideGCCollisionSetupForTraces == GCCSFT_Property) || (ForceOverrideGCCollisionSetupForTraces == GCCSFT_ForceSM));
	if (Parameters.bUseStaticMeshCollisionForTraces != bUseSMCollisionForTraces && CreateTraceCollisionGeometryCallback != nullptr)
	{
		const int32 TransformIndex = Parameters.InitialRootIndex;
		const int32 ParticleIndex = FromTransformToParticleIndex[TransformIndex];
		if (GTParticles.IsValidIndex(ParticleIndex))
		{
			FParticle* P = GTParticles[ParticleIndex].Get();
			if (P != nullptr)
			{
				if (bUseSMCollisionForTraces)
				{
					const TManagedArray<FTransform>& MassToLocal = Parameters.RestCollectionShared->GetAttribute<FTransform>("MassToLocal", FGeometryCollection::TransformGroup);

					const FTransform ToLocal = MassToLocal[TransformIndex].Inverse();
					TArray<Chaos::FImplicitObjectPtr> Geoms;
					Chaos::FShapesArray Shapes;
					CreateTraceCollisionGeometryCallback(ToLocal, Geoms, Shapes);

					Chaos::FImplicitObjectPtr ImplicitGeometry = MakeImplicitObjectPtr<Chaos::FImplicitObjectUnion>(MoveTemp(Geoms));
					P->SetGeometry(ImplicitGeometry);
				}
				else
				{
					const FVector Scale = WorldTransform_External.GetScale3D();
					TManagedArray<Chaos::FImplicitObjectPtr>& Implicits = GameThreadCollection.ModifyAttribute<Chaos::FImplicitObjectPtr>(FGeometryDynamicCollection::ImplicitsAttribute, FTransformCollection::TransformGroup);
					Chaos::FImplicitObjectPtr ImplicitGeometry = Implicits[TransformIndex];
					if (ImplicitGeometry && !Scale.Equals(FVector::OneVector))
					{
						ImplicitGeometry = ImplicitGeometry->CopyGeometryWithScale(Scale);
					}
					P->SetGeometry(ImplicitGeometry);
				}
			}
			// We only want to change the value of our boolean if we actually changed the collision, so the collision state matches
			// note we can set thsi member of Parameters on the GT because it is only used and set on the GT 
			Parameters.bUseStaticMeshCollisionForTraces = bUseSMCollisionForTraces;
		}
	}
}

void FGeometryCollectionPhysicsProxy::SetDamageThresholds_External(const TArray<float>& DamageThresholds)
{
	ExecuteOnPhysicsThread(*this,
		[this, DamageThresholds]()
		{
			SetDamageThresholds_Internal(DamageThresholds);
		});
}

void FGeometryCollectionPhysicsProxy::SetDamageThresholds_Internal(const TArray<float>& DamageThresholds)
{
	Parameters.DamageThreshold = DamageThresholds;

	UpdateDamageThreshold_Internal();
}

void FGeometryCollectionPhysicsProxy::SetDamagePropagationData_External(bool bEnabled, float BreakDamagePropagationFactor, float ShockDamagePropagationFactor)
{
	ExecuteOnPhysicsThread(*this,
		[this, bEnabled, BreakDamagePropagationFactor, ShockDamagePropagationFactor]()
		{
			SetDamagePropagationData_Internal(bEnabled, BreakDamagePropagationFactor, ShockDamagePropagationFactor);
		});
}

void FGeometryCollectionPhysicsProxy::SetDamagePropagationData_Internal(bool bEnabled, float BreakDamagePropagationFactor, float ShockDamagePropagationFactor)
{
	Parameters.bUseDamagePropagation = bEnabled;
	Parameters.BreakDamagePropagationFactor = BreakDamagePropagationFactor;
	Parameters.ShockDamagePropagationFactor = ShockDamagePropagationFactor;
}

void FGeometryCollectionPhysicsProxy::SetDamageModel_External(EDamageModelTypeEnum DamageModel)
{
	ExecuteOnPhysicsThread(*this,
		[this, DamageModel]()
		{
			SetDamageModel_Internal(DamageModel);
		});
}

void FGeometryCollectionPhysicsProxy::SetDamageModel_Internal(EDamageModelTypeEnum DamageModel)
{
	Parameters.DamageModel = DamageModel;
	Parameters.DamageEvaluationModel = GetDamageEvaluationModel(Parameters.DamageModel);

	UpdateDamageThreshold_Internal();
}

void FGeometryCollectionPhysicsProxy::SetUseMaterialDamageModifiers_External(bool bUseMaterialDamageModifiers)
{
	ExecuteOnPhysicsThread(*this,
		[this, bUseMaterialDamageModifiers]()
		{
			SetUseMaterialDamageModifiers_Internal(bUseMaterialDamageModifiers);
		});
}

void FGeometryCollectionPhysicsProxy::SetUseMaterialDamageModifiers_Internal(bool bUseMaterialDamageModifiers)
{
	Parameters.bUseMaterialDamageModifiers = bUseMaterialDamageModifiers;

	UpdateDamageThreshold_Internal();
}

void FGeometryCollectionPhysicsProxy::SetMaterialOverrideMassScaleMultiplier_External(float InMultiplier)
{
	ExecuteOnPhysicsThread(*this,
		[this, InMultiplier]()
		{
			SetMaterialOverrideMassScaleMultiplier_Internal(InMultiplier);
		});
}

void FGeometryCollectionPhysicsProxy::SetMaterialOverrideMassScaleMultiplier_Internal(float InMultiplier)
{
	const float NewValue = InMultiplier;
	const float OldValue = Parameters.MaterialOverrideMassScaleMultiplier;

	// Because we need to send a change in scale , we need to make sure the physics state has been created
	// otherwise we set the change to 1.0 and let the PT pick up the right Parameters.MaterialOverrideMassScaleMultiplier during InitializeBodiesPT
	float MaterialOverrideMassScaleMultiplierChange = 1.0;
	if (bIsInitializedOnPhysicsThread && OldValue > SMALL_NUMBER)
	{
		MaterialOverrideMassScaleMultiplierChange = NewValue / OldValue;
	}
	Parameters.MaterialOverrideMassScaleMultiplier = NewValue;

	for (int32 ParticleIndex = 0; ParticleIndex < SolverParticleHandles.Num(); ++ParticleIndex)
	{
		if (Chaos::FPBDRigidClusteredParticleHandle* Handle = SolverParticleHandles[ParticleIndex])
		{
			const Chaos::FReal NewM = Handle->M() * MaterialOverrideMassScaleMultiplierChange;
			const Chaos::FVec3 NewI = Handle->I() * MaterialOverrideMassScaleMultiplierChange;

			Handle->SetM(NewM);
			Handle->SetI(NewI);
			const Chaos::FReal InvM = (NewM > 0.0f) ? 1.0f / NewM : 0.0f;
			const Chaos::FVec3 InvI = (NewM > 0.0f) ? Chaos::FVec3(NewI).Reciprocal() : Chaos::FVec3::ZeroVector;
			Handle->SetInvM(InvM);
			Handle->SetInvI(InvI);
		}
	}
}

void FGeometryCollectionPhysicsProxy::SetEnableGravity_External(bool bEnabled)
{
	for (TUniquePtr<FParticle>& GTParticle : GTParticles)
	{
		if (GTParticle)
		{
			GTParticle->SetGravityEnabled(bEnabled);
		}
	}

	ExecuteOnPhysicsThread(*this, 
		[this, bEnabled]() 
		{
			SetEnableGravity_Internal(bEnabled);
		});
}

void FGeometryCollectionPhysicsProxy::SetEnableGravity_Internal(bool bEnabled)
{
	Parameters.GravityGroupIndex = bEnabled;

	if (Chaos::FPhysicsSolver* RBDSolver = GetSolver<Chaos::FPhysicsSolver>())
	{
		if (Chaos::FPBDRigidsSolver::FPBDRigidsEvolution* Evolution = RBDSolver->GetEvolution())
		{
			for (Chaos::FPBDRigidClusteredParticleHandle* Handle : SolverParticleHandles)
			{
				if (Handle)
				{
					Handle->SetGravityEnabled(bEnabled);
					if (!Handle->Disabled())
					{
						Evolution->WakeParticle(Handle);
					}
				}
			}
		}
	}
}

void FGeometryCollectionPhysicsProxy::SetGravityGroupIndex_External(int32 GravityGroupIndex)
{
	ExecuteOnPhysicsThread(*this, 
		[this, GravityGroupIndex]() 
		{ 
			SetGravityGroupIndex_Internal(GravityGroupIndex); 
		});
}

void FGeometryCollectionPhysicsProxy::SetGravityGroupIndex_Internal(int32 GravityGroupIndex)
{
	Parameters.GravityGroupIndex = GravityGroupIndex;

	for (Chaos::FPBDRigidClusteredParticleHandle* Handle : SolverParticleHandles)
	{
		if (Handle)
		{
			Handle->SetGravityGroupIndex(Parameters.GravityGroupIndex);
		}
	}
}

void FGeometryCollectionPhysicsProxy::SetOneWayInteractionLevel_External(int32 InOneWayInteractionLevel)
{
	// @todo(chaos): not sure we need the OneWayInteraction flag on the GT side for geometry collection particles?
	const TManagedArray<int32>& Level = Parameters.RestCollectionShared->GetAttribute<int32>(LevelAttributeName, FTransformCollection::TransformGroup);
	for (int32 ParticleIndex = 0; ParticleIndex < GTParticles.Num(); ++ParticleIndex)
	{
		TUniquePtr<FParticle>& Particle = GTParticles[ParticleIndex];
		if (Particle.IsValid())
		{
			const bool bIsOneWayInteraction = (InOneWayInteractionLevel >= 0) && (Level[FromParticleToTransformIndex[ParticleIndex]] >= InOneWayInteractionLevel);
			Particle->SetOneWayInteraction(bIsOneWayInteraction);
		}
	}

	ExecuteOnPhysicsThread(*this, [this, InOneWayInteractionLevel]()
		{
			SetOneWayInteractionLevel_Internal(InOneWayInteractionLevel);
		});
}

void FGeometryCollectionPhysicsProxy::SetOneWayInteractionLevel_Internal(int32 InOneWayInteractionLevel)
{
	Parameters.OneWayInteractionLevel = InOneWayInteractionLevel;

	const TManagedArray<int32>& Level = PhysicsThreadCollection.GetInitialLevels().Get();
	for (int32 ParticleIndex = 0; ParticleIndex < SolverParticleHandles.Num(); ++ParticleIndex)
	{
		if (Chaos::FPBDRigidClusteredParticleHandle* Handle = SolverParticleHandles[ParticleIndex])
		{
			const bool bIsOneWayInteraction = (Parameters.OneWayInteractionLevel >= 0) && (Level[FromParticleToTransformIndex[ParticleIndex]] >= Parameters.OneWayInteractionLevel);
			Handle->SetOneWayInteraction(bIsOneWayInteraction);
		}
	}
}

void FGeometryCollectionPhysicsProxy::SetPhysicsMaterial_External(const Chaos::FMaterialHandle& MaterialHandle)
{
	// update GT particles
	for (const TUniquePtr<FParticle>& GTParticle : GTParticles)
	{
		if (GTParticle)
		{
			for (const TUniquePtr<Chaos::FPerShapeData>& Shape : GTParticle->ShapesArray())
			{
				Shape->SetMaterial(MaterialHandle);
			}
		}
	}

	// update physics thread
	ExecuteOnPhysicsThread(*this, [this, MaterialHandle]()
		{
			SetPhysicsMaterial_Internal(MaterialHandle);
		});
}

void FGeometryCollectionPhysicsProxy::SetPhysicsMaterial_Internal(const Chaos::FMaterialHandle& MaterialHandle)
{
	using namespace Chaos;
	if (Parameters.PhysicalMaterialHandle != MaterialHandle)
	{
		Parameters.PhysicalMaterialHandle = MaterialHandle;

		// set materials on the particles
		FChaosPhysicsMaterial* SolverMaterial = nullptr;
		if (Chaos::FPhysicsSolver* RigidSolver = GetSolver<Chaos::FPhysicsSolver>())
		{
			// #BGTODO - non-updating parameters - remove lin/ang drag arrays and always query material if this stays a material parameter
			SolverMaterial = RigidSolver->GetSimMaterials().Get(Parameters.PhysicalMaterialHandle.InnerHandle);
		}
		if (SolverMaterial)
		{
			for (Chaos::FPBDRigidClusteredParticleHandle* Handle : SolverParticleHandles)
			{
				if (Handle)
				{
					Handle->SetLinearEtherDrag(SolverMaterial->LinearEtherDrag);
					Handle->SetAngularEtherDrag(SolverMaterial->AngularEtherDrag);

					const Chaos::FShapesArray& Shapes = Handle->ShapesArray();
					for (const TUniquePtr<Chaos::FPerShapeData>& Shape : Shapes)
					{
						Shape->SetMaterial(Parameters.PhysicalMaterialHandle);
					}
				}
			}
		}

		// adjust damage threshold as they depends on the material DamageThresholdMultiplier or strength
		UpdateDamageThreshold_Internal();
	}
}

void FGeometryCollectionPhysicsProxy::SetLinearDamping_External(float InDamping)
{
	// update physics thread
	ExecuteOnPhysicsThread(*this, [this, InDamping]()
		{
			SetLinearDamping_Internal(InDamping);
		});
}

void FGeometryCollectionPhysicsProxy::SetLinearDamping_Internal(float InDamping)
{
	Parameters.LinearDamping = InDamping;

	for (Chaos::FPBDRigidClusteredParticleHandle* Handle : SolverParticleHandles)
	{
		if (Handle)
		{
			Handle->SetLinearEtherDrag(Parameters.LinearDamping);
			// make sure we update internal clusters parent
			if (Chaos::FPBDRigidClusteredParticleHandle* ParentHandle = Handle->Parent())
			{
				if (ParentHandle->InternalCluster() && ParentHandle->PhysicsProxy() == Handle->PhysicsProxy())
				{
					ParentHandle->SetLinearEtherDrag(Parameters.LinearDamping);
				}
			}
		}
	}
}

void FGeometryCollectionPhysicsProxy::SetLinearDampingPerTransform_External(const TArray<int32>& TransformIndices, float InDamping)
{
	// update physics thread
	ExecuteOnPhysicsThread(*this, [this, InDamping, TransformIndices]()
		{
			SetLinearDampingPerTransform_Internal(TransformIndices, InDamping);
		});
}

void FGeometryCollectionPhysicsProxy::SetLinearDampingPerTransform_Internal(const TArray<int32>& TransformIndices, float InDamping)
{
	for (const int32 TransformIndex: TransformIndices)
	{
		if (FromTransformToParticleIndex.IsValidIndex(TransformIndex))
		{
			const int32 ParticleIndex = FromTransformToParticleIndex[TransformIndex];
			if (Chaos::FPBDRigidClusteredParticleHandle* Handle = SolverParticleHandles.IsValidIndex(ParticleIndex) ? SolverParticleHandles[ParticleIndex] : nullptr)
			{
				Handle->SetLinearEtherDrag(InDamping);
			}
		}
	}
}

void FGeometryCollectionPhysicsProxy::SetAngularDamping_External(float InDamping)
{
	// update physics thread
	ExecuteOnPhysicsThread(*this, [this, InDamping]()
		{
			SetAngularDamping_Internal(InDamping);
		});
}

void FGeometryCollectionPhysicsProxy::SetAngularDamping_Internal(float InDamping)
{
	Parameters.AngularDamping = InDamping;

	for (Chaos::FPBDRigidClusteredParticleHandle* Handle : SolverParticleHandles)
	{
		if (Handle)
		{
			Handle->SetAngularEtherDrag(Parameters.AngularDamping);
			// make sure we update internal clusters parent
			if (Chaos::FPBDRigidClusteredParticleHandle* ParentHandle = Handle->Parent())
			{
				if (ParentHandle->InternalCluster() && ParentHandle->PhysicsProxy() == Handle->PhysicsProxy())
				{
					ParentHandle->SetAngularEtherDrag(Parameters.AngularDamping);
				}
			}
		}
	}
}

void FGeometryCollectionPhysicsProxy::SetAngularDampingPerTransform_External(const TArray<int32>& TransformIndices, float InDamping)
{
	// update physics thread
	ExecuteOnPhysicsThread(*this, [this, InDamping, TransformIndices]()
		{
			SetAngularDampingPerTransform_Internal(TransformIndices, InDamping);
		});
}

void FGeometryCollectionPhysicsProxy::SetAngularDampingPerTransform_Internal(const TArray<int32>& TransformIndices, float InDamping)
{
	for (const int32 TransformIndex : TransformIndices)
	{
		if (FromTransformToParticleIndex.IsValidIndex(TransformIndex))
		{
			const int32 ParticleIndex = FromTransformToParticleIndex[TransformIndex];
			if (Chaos::FPBDRigidClusteredParticleHandle* Handle = SolverParticleHandles.IsValidIndex(ParticleIndex) ? SolverParticleHandles[ParticleIndex] : nullptr)
			{
				Handle->SetAngularEtherDrag(InDamping);
			}
		}
	}
}

void FGeometryCollectionPhysicsProxy::PushStateOnGameThread(Chaos::FPBDRigidsSolver* InSolver)
{
	// CONTEXT: ANYTHREAD but spawned form GAMETHREAD in a parallelFor
	// nothing to do 
}

void FGeometryCollectionPhysicsProxy::PushToPhysicsState()
{
	using namespace Chaos;
	// CONTEXT: PHYSICSTHREAD
}

void FGeometryCollectionPhysicsProxy::SetClusteredParticleKinematicTarget_Internal(Chaos::FPBDRigidClusteredParticleHandle* Handle, const FTransform& NewWorldTransform)
{
	// CONTEXT: PHYSICSTHREAD
	// this should be called only on the physics thread
	const Chaos::EObjectStateType ObjectState = Handle->ObjectState();
	if (ensure(ObjectState == Chaos::EObjectStateType::Kinematic))
	{
		Chaos::FKinematicTarget NewKinematicTarget;
		NewKinematicTarget.SetTargetMode(NewWorldTransform);

		if (Chaos::FPhysicsSolver* RBDSolver = GetSolver<Chaos::FPhysicsSolver>())
		{
			RBDSolver->GetEvolution()->SetParticleKinematicTarget(Handle, NewKinematicTarget);
			RBDSolver->GetEvolution()->DirtyParticle(*Handle);
		}
	}
}

static EObjectStateTypeEnum GetObjectStateEnumFromObjectState(Chaos::EObjectStateType State)
{
	switch (State)
	{
	case Chaos::EObjectStateType::Kinematic:
		return EObjectStateTypeEnum::Chaos_Object_Kinematic;
	case Chaos::EObjectStateType::Static:
		return EObjectStateTypeEnum::Chaos_Object_Static;
	case Chaos::EObjectStateType::Sleeping:
		return EObjectStateTypeEnum::Chaos_Object_Sleeping;
	case Chaos::EObjectStateType::Dynamic:
	case Chaos::EObjectStateType::Uninitialized:
	default:
		return EObjectStateTypeEnum::Chaos_Object_Dynamic;
	}
}

static EObjectStateTypeEnum GetObjectStateFromHandle(const Chaos::TPBDRigidClusteredParticleHandle<Chaos::FReal, 3>* Handle)
{
	if (!Handle->Sleeping())
	{
		return GetObjectStateEnumFromObjectState(Handle->ObjectState());
	}
	return EObjectStateTypeEnum::Chaos_Object_Sleeping;
}

void FGeometryCollectionPhysicsProxy::PrepareBufferData(Chaos::FDirtyGeometryCollectionData& BufferData, const FGeometryDynamicCollection& ThreadCollection,  Chaos::FReal SolverLastDt)
{
	/**
	* CONTEXT: GAMETHREAD or PHYSICS THREAD
	* this method needs to be careful of data access , ultimately only the this pointer can be passed to the BufferData 
	*/
	BufferData.SetProxy(*this);

	IsObjectDynamic = false;
	FGeometryCollectionResults& Results = BufferData.Results();
	Results.SetSolverDt(SolverLastDt);
	Results.InitArrays(ThreadCollection);
}

void FGeometryCollectionPhysicsProxy::BufferPhysicsResults_External(Chaos::FDirtyGeometryCollectionData& BufferData)
{
	/**
	* CONTEXT: GAMETHREAD
	* Called per-tick when async is on after the simulation has completed.
	* goal is collect current game thread data of the proxy so it can be used if no previous physics thread data is available for interpolating  
	*/
	
	// for geometry collection the physics thread is the authority
	// we leave the results empty, meaning that nothing has changed 
	// PullFromPhysics state can then use the GT state to interpolate from if necessary
	PrepareBufferData(BufferData, PhysicsThreadCollection);
}

void FGeometryCollectionPhysicsProxy::FlushDirtyParticles(Chaos::FPBDRigidsSolver& CurrentSolver)
{
	for (TConstSetBitIterator<> It(ParticlesToDirty); It; ++It)
	{
		if (FClusterHandle* Handle = SolverParticleHandles[It.GetIndex()])
		{
			CurrentSolver.GetEvolution()->DirtyParticle(*Handle);
		}
	}
	ParticlesToDirty.Init(false, ParticlesToDirty.Num());
}

static void UpdateParticleHandleTransform(Chaos::FPBDRigidsSolver& CurrentSolver, Chaos::FPBDRigidClusteredParticleHandle& Handle, const Chaos::FRigidTransform3& NewTransform, TBitArray<>& ParticlesToDirty, int32 ParticleIndex)
{
	Handle.SetX(NewTransform.GetTranslation());
	Handle.SetR(NewTransform.GetRotation());
	Handle.UpdateWorldSpaceState(NewTransform, Chaos::FVec3{0});

	ParticlesToDirty[ParticleIndex] = true;
}

static void UpdateParticleHandleTransformIfNeeded(Chaos::FPBDRigidsSolver& CurrentSolver, Chaos::FPBDRigidClusteredParticleHandle& Handle, const Chaos::FRigidTransform3& NewTransform, TBitArray<>& ParticlesToDirty, int32 ParticleIndex)
{
	const Chaos::FVec3 NewX = NewTransform.GetTranslation();
	const Chaos::FVec3 OldX = Handle.GetX();
	
	const Chaos::FRotation3 NewR  = NewTransform.GetRotation();
	const Chaos::FRotation3 OldR = Handle.GetR();
	
	if (OldX != NewX || OldR != NewR)
	{
		UpdateParticleHandleTransform(CurrentSolver, Handle, NewTransform, ParticlesToDirty, ParticleIndex);
	}
}

// remove a transform from its parent in the physics thread collection
// return true if the removal was successfull ( false if the child has already no parent )
inline static bool RemoveFromParentInCollection_Internal(FGeometryDynamicCollection& PhysicsThreadCollection, int32 TransformGroupIndex)
{
	// CONTEXT : PHYSICS THREAD
	if (int32 ParentIndex = PhysicsThreadCollection.GetParent(TransformGroupIndex); ParentIndex != INDEX_NONE)
	{
		PhysicsThreadCollection.SetHasParent(TransformGroupIndex, false);
		return true;
	}
	return false;
}

void FGeometryCollectionPhysicsProxy::BufferPhysicsResults_Internal(Chaos::FPBDRigidsSolver* CurrentSolver, Chaos::FDirtyGeometryCollectionData& BufferData)
{
	/**
	 * CONTEXT: PHYSICSTHREAD
	 * Called per-tick after the simulation has completed. The proxy should cache the results of their
	 * simulation into the local buffer. 
	 */
	using namespace Chaos;
	SCOPE_CYCLE_COUNTER(STAT_CacheResultGeomCollection);
	if (IsObjectDeleting) return;

	//todo: should this use timestamp instead of GetLastDt for async mode?
	PrepareBufferData(BufferData, PhysicsThreadCollection, CurrentSolver->GetLastDt());

	IsObjectDynamic = false;
	FGeometryCollectionResults& Results = BufferData.Results();

	const FTransform& ActorToWorld = Parameters.WorldTransform;
	const bool IsActorScaled = !ActorToWorld.GetScale3D().Equals(FVector::OneVector);
	const FTransform ActorScaleTransform(FQuat::Identity,  FVector::ZeroVector, ActorToWorld.GetScale3D());

	UniqueIdxToInternalClusterHandle.Reset();
	
	check(NumTransforms == PhysicsThreadCollection.GetNumTransforms());
	if(NumTransforms > 0)
	{ 
		SCOPE_CYCLE_COUNTER(STAT_CalcParticleToWorld);

		const TManagedArray<FTransform>& MassToLocalArray = Parameters.RestCollectionShared->GetAttribute<FTransform>(MassToLocalAttributeName, FTransformCollection::TransformGroup);

		// Explicitly handle root breaking to ensure that the Root GT particle Active state matches the PT particle.
		// If the root was in a cluster union and the cluster gets broken immediately on removal, we
		// will never hit the logic below that checks for the particle changing from active to inactive.
		// The parent checks also miss the case where the parent has an external parent (GC particles
		// historically assume that the root has no parent which is not true any more).
		if (Parameters.InitialRootIndex != INDEX_NONE && FromTransformToParticleIndex[Parameters.InitialRootIndex] != INDEX_NONE)
		{
			const Chaos::FPBDRigidClusteredParticleHandle* RootHandle = SolverParticleHandles[FromTransformToParticleIndex[Parameters.InitialRootIndex]];
			if (RootHandle != nullptr)
			{
				const bool bIsRootBroken = RootHandle->Disabled() && (RootHandle->Parent() == nullptr);
				Results.IsRootBroken = bIsRootBroken;
			}
		}

		for (int32 ParticleIndex = 0; ParticleIndex < NumEffectiveParticles; ++ParticleIndex)
		{
			const int32 TransformGroupIndex = FromParticleToTransformIndex[ParticleIndex];
			Chaos::FPBDRigidClusteredParticleHandle* Handle = SolverParticleHandles[ParticleIndex];
			if (!Handle)
			{
				PhysicsThreadCollection.Active[TransformGroupIndex] = false;
				continue;
			}

			const bool bWasActive = PhysicsThreadCollection.Active[TransformGroupIndex];
			const bool bIsActive = !Handle->Disabled();

			FGeometryCollectionResults::FPositionData PositionData;
			PositionData.ParticleX = Handle->GetX();
			PositionData.ParticleR = Handle->GetR();

			FGeometryCollectionResults::FVelocityData VelocityData;
			VelocityData.ParticleV = Handle->GetV();
			VelocityData.ParticleW = Handle->GetW();

			FGeometryCollectionResults::FStateData StateData;
			StateData.TransformIndex = TransformGroupIndex;
			StateData.HasParent = PhysicsThreadCollection.GetHasParent(TransformGroupIndex);
			StateData.InternalClusterUniqueIdx = INDEX_NONE;
			StateData.State.DisabledState = Handle->Disabled(); // this can change if the particle has a cluster union parent 
			StateData.State.HasDecayed = Handle->Disabled() && (Handle->Parent() == nullptr);
			StateData.State.HasInternalClusterParent = false;
			StateData.State.DynamicInternalClusterParent = false;
			StateData.State.HasClusterUnionParent = false;
			StateData.State.DynamicState = static_cast<int8>(GetObjectStateFromHandle(Handle));

			bool bHasChanged = false;

			// Update the transform and parent hierarchy of the active rigid bodies. Active bodies can be either
			// rigid geometry defined from the leaf nodes of the collection, or cluster bodies that drive an entire
			// branch of the hierarchy within the GeometryCollection.
			// - Active bodies are directly driven from the global position of the corresponding
			//   rigid bodies within the solver 
			// - Deactivated bodies are driven from the transforms of their active parents. However the solver can
			//   take ownership of the parents during the simulation, so it might be necessary to force deactivated
			//   bodies out of the collections hierarchy during the simulation.  

			if (bIsActive || (bWasActive != bIsActive))
			{
				bHasChanged = true;
			}
					
			Chaos::FPBDRigidClusteredParticleHandle* ClusterParent = Handle->Parent();

			// Has parent changed? ( can be a new one or nullptr)
			const bool bHasNewParent = (SolverClusterID[ParticleIndex] != ClusterParent) || (!ClusterParent && PhysicsThreadCollection.GetHasParent(TransformGroupIndex));
			if (bHasNewParent)
			{
				// Force all driven rigid bodies out of the transform hierarchy ( because the new parent can only be an internal cluster )
				if (RemoveFromParentInCollection_Internal(PhysicsThreadCollection, TransformGroupIndex))
				{
					// If the parent flag of this NON DISABLED body is set to false,
					// then it was just unparented, likely either by rigid clustering or by fields.  We
					// need to force all such enabled rigid bodies out of the transform hierarchy.
					StateData.HasParent = false;

					// Indicate that this object needs to be updated and the proxy is active.
					StateData.State.DisabledState = false;

					bHasChanged = true;
				}
				// set new parent ( likely an internal cluster )
				SolverClusterID[ParticleIndex] = ClusterParent;
			}

			// do we have an internal cluster parent ( it also implies we are a disabled particle )
			if (ClusterParent && ClusterParent->InternalCluster())
			{
				const bool bIsOwnedInternalCluster = ClusterParent->PhysicsProxy() == this;

				// We don't want the to keep track of any external internal clusters (i.e. cluster unions).
				if (bIsOwnedInternalCluster)
				{
					StateData.InternalClusterUniqueIdx = ClusterParent->UniqueIdx().Idx;
					if (!UniqueIdxToInternalClusterHandle.Contains(StateData.InternalClusterUniqueIdx))
					{
						UniqueIdxToInternalClusterHandle.Add(StateData.InternalClusterUniqueIdx, ClusterParent);
					}
				}

				// child of internal clusters may not have their position up to date 
				// so we need to compute it from the ChildToParent transform 
				const FTransform ParticleToWorld = Handle->ChildToParent() * FRigidTransform3(ClusterParent->GetX(), ClusterParent->GetR()); // aka ClusterChildToWorld
				PositionData.ParticleX = ParticleToWorld.GetTranslation();
				PositionData.ParticleR = ParticleToWorld.GetRotation();
				
				// Indicate that this object needs to be updated and the proxy is active.
				StateData.State.DisabledState = bPropagateInternalClusterDisableFlagToChildren ? ClusterParent->Disabled() : false;
				StateData.State.HasInternalClusterParent = true;
				StateData.State.DynamicInternalClusterParent = (ClusterParent->IsDynamic());
				StateData.State.HasClusterUnionParent = ClusterParent->PhysicsProxy()->GetType() == EPhysicsProxyType::ClusterUnionProxy;

				bHasChanged = true;

				// as we just transitioned from disabled to non disabled the update is unconditional 
				UpdateParticleHandleTransform(*CurrentSolver, *Handle, ParticleToWorld, ParticlesToDirty, ParticleIndex);
			}

			if (bGeometryCollectionEnabledNestedChildTransformUpdates)
			{
				if (ClusterParent && !ClusterParent->Disabled())
				{
					const FRigidTransform3 ChildToWorld = Handle->ChildToParent() * FRigidTransform3(ClusterParent->GetX(), ClusterParent->GetR());
					UpdateParticleHandleTransformIfNeeded(*CurrentSolver, *Handle, ChildToWorld, ParticlesToDirty, ParticleIndex);
					// fields may have applied velocities, we need to make sure to clear that up, so that we don't accumulate
					VelocityData.ParticleV = FVec3::ZeroVector;
					VelocityData.ParticleW = FVec3::ZeroVector;

					bHasChanged = true;
				}
			}

			// Add to the results if anything has been marked as changed 
			if (bHasChanged)
			{
				// default is what we have on the collection
				const FTransform3f& ParentSpaceTransform3f = PhysicsThreadCollection.GetTransform(TransformGroupIndex);
				FTransform ParentSpaceTransform = FTransform(ParentSpaceTransform3f);

				// recompute parent space transform if there's no more parent or if the parent is an internal cluster 
				if (!ClusterParent || ClusterParent->InternalCluster())
				{
					const FRigidTransform3 ParticleToWorld(PositionData.ParticleX, PositionData.ParticleR);
					const FTransform MassToLocal = MassToLocalArray[TransformGroupIndex];
					ParentSpaceTransform = MassToLocal.GetRelativeTransformReverse(ParticleToWorld).GetRelativeTransform(ActorToWorld);
					ParentSpaceTransform.NormalizeRotation();
					if (IsActorScaled)
					{
						ParentSpaceTransform = MassToLocal.Inverse() * ActorScaleTransform * MassToLocal * ParentSpaceTransform;
					}
				}

				const int32 EntryIndex = Results.AddEntry(TransformGroupIndex);
				Results.SetState(EntryIndex, StateData);
				Results.SetPositions(EntryIndex, PositionData);
				Results.SetVelocities(EntryIndex, VelocityData);
				FTransform3f NewParentSpaceTransform3f(ParentSpaceTransform);
				if (!NewParentSpaceTransform3f.Equals(ParentSpaceTransform3f))
				{
					PhysicsThreadCollection.SetTransform(TransformGroupIndex, NewParentSpaceTransform3f);
				}
				IsObjectDynamic = true;
			}

			// update collection active state 
			PhysicsThreadCollection.Active[TransformGroupIndex] = !StateData.State.DisabledState;

#if WITH_EDITORONLY_DATA
			// collect damage information to display in the UI 
			FGeometryCollectionResults::FDamageData DamageData;
			DamageData.Damage = FMath::Max(Handle->CollisionImpulse(), Handle->GetExternalStrain());
			DamageData.DamageThreshold = Handle->GetInternalStrains();
			Results.SetDamages(TransformGroupIndex, DamageData);
#endif			
		}    // end for
	}        // STAT_CalcParticleToWorld scope
	
	// Advertise to game thread
	Results.IsObjectDynamic = IsObjectDynamic;
	Results.IsObjectLoading = IsObjectLoading;

	//we are now done with this physics thread tick , we can set the previous world transform 
	Parameters.PrevWorldTransform = Parameters.WorldTransform;
}

void FGeometryCollectionPhysicsProxy::FlipBuffer()
{
	/**
	 * CONTEXT: PHYSICSTHREAD (Write Locked)
	 * Called by the physics thread to signal that it is safe to perform any double-buffer flips here.
	 * The physics thread has pre-locked an RW lock for this operation so the game thread won't be reading
	 * the data
	 */

	PhysToGameInterchange.FlipProducer();
}

// Update game thread particle X and R properties and retun true if they changed from previous values
static inline bool UpdateGTParticleXR(Chaos::FPBDRigidParticle& GTParticle, const Chaos::FVec3& NewX, const Chaos::FRotation3& NewR, bool bForceUpdateActiveTransforms)
{
	GC_PHYSICSPROXY_CHECK_FOR_NAN(NewX);

	const Chaos::FVec3 OldX = GTParticle.GetX();
	const bool bNeedUpdateX = bForceUpdateActiveTransforms || (!NewX.Equals(OldX, GeometryCollectionPositionUpdateTolerance));
	if (bNeedUpdateX)
	{
		GTParticle.SetX(NewX, false);
	}

	const Chaos::FRotation3 OldR = GTParticle.R();
	const bool bNeedUpdateR = bForceUpdateActiveTransforms || (!NewR.Equals(OldR, GeometryCollectionRotationUpdateTolerance));
	if (bNeedUpdateR)
	{
		GTParticle.SetR(NewR, false);
	}

	const bool bChanged = (bNeedUpdateX || bNeedUpdateR);
	if (bChanged)
	{
		GTParticle.UpdateShapeBounds();
	}

	return bChanged;
}

// Update game thread particle V and W properties and retun true if they changed from previous values
static inline bool UpdateGTParticleVW(Chaos::FPBDRigidParticle& GTParticle, const Chaos::FVec3& NewV, const Chaos::FVec3& NewW)
{
	GC_PHYSICSPROXY_CHECK_FOR_NAN(NewX);

	const Chaos::FVec3 OldV = GTParticle.GetV();
	const bool bNeedUpdateV = (!NewV.Equals(OldV, GeometryCollectionLinearVelocityUpdateTolerance));
	if (bNeedUpdateV)
	{
		GTParticle.SetV(NewV, false);
	}

	const Chaos::FVec3 OldW = GTParticle.GetW();
	const bool bNeedUpdateW = (!NewW.Equals(OldW, GeometryCollectionAngularVelocityUpdateTolerance));
	if (bNeedUpdateW)
	{
		GTParticle.SetW(NewW, false);
	}
	return bNeedUpdateV || bNeedUpdateW;
}

// Update a value and return true if the value was different from the old one 
template <typename TOld, typename TNew>
static inline bool UpdateValue(TOld& ValueInOut,const TNew& NewValue)
{
	const bool bValueDifferent = (NewValue != ValueInOut);
	if (bValueDifferent)
	{
		ValueInOut = NewValue;
	}
	return bValueDifferent;
}

static inline bool UpdateValue(FBitReference ValueInOut, const bool& NewValue)
{
	const bool bValueDifferent = (NewValue != ValueInOut);
	if (bValueDifferent)
	{
		ValueInOut = NewValue;
	}
	return bValueDifferent;
}

static inline bool UpdateTransform(FTransform& ValueInOut, const FTransform& NewValue)
{
	const bool bValueDifferent = !NewValue.Equals(ValueInOut);
	if (bValueDifferent)
	{
		ValueInOut = NewValue;
	}
	return bValueDifferent;
}

// this helper class helps getting the right interpolated values )
// it also handles the case where entries for a specific transform index may be available in one or the other the other results 
// and properly fallback to interpolate 
struct FResultInterpolator
{
public:
	FResultInterpolator(
		int32 TransformIndexIn,
		const FGeometryCollectionResults& ResultsIn, 
		const FGeometryCollectionResults& NextResultsIn,
		Chaos::FPBDRigidParticle& GTParticleIn,
		Chaos::FRealSingle AlphaIn
	)
		: TransformIndex(TransformIndexIn)
		, Results(&ResultsIn)
		, NextResults(&NextResultsIn)
		, GTParticle(GTParticleIn)
		, Alpha(AlphaIn)
	{
		EntryIndex = Results->GetEntryIndexByTransformIndex(TransformIndexIn);
		NextEntryIndex = NextResults->GetEntryIndexByTransformIndex(TransformIndexIn);
	}

	bool HasNoEntry() const
	{
		return (EntryIndex == INDEX_NONE && NextEntryIndex == INDEX_NONE);
	}

	void GetPositions(Chaos::FVec3& PosOut, Chaos::FRotation3& RotOut) const
	{
		if (NextEntryIndex == INDEX_NONE)
		{
			const FGeometryCollectionResults::FPositionData& Data = Results->GetPositions(EntryIndex);
			// not present in next results just use results
			PosOut = Data.ParticleX;
			RotOut = Data.ParticleR;
		}
		else if (EntryIndex == INDEX_NONE)
		{
			// not present in current results, just use next results
			const FGeometryCollectionResults::FPositionData& NextData = NextResults->GetPositions(NextEntryIndex);
			PosOut = NextData.ParticleX;
			RotOut = NextData.ParticleR;
		}
		else
		{
			// both available 
			const FGeometryCollectionResults::FPositionData& Data = Results->GetPositions(EntryIndex);
			const FGeometryCollectionResults::FPositionData& NextData = NextResults->GetPositions(NextEntryIndex);
			PosOut = FMath::Lerp(Data.ParticleX, NextData.ParticleX, Alpha);
			RotOut = FQuat::Slerp(Data.ParticleR, NextData.ParticleR, Alpha);
		}
	}

	void GetVelocities(FVector3f& LinearVelOut, FVector3f& AngularVelOut) const
	{
		if (NextEntryIndex == INDEX_NONE)
		{
			// not present in next results just use results
			const FGeometryCollectionResults::FVelocityData& Data = Results->GetVelocities(EntryIndex);
			LinearVelOut = Data.ParticleV;
			AngularVelOut = Data.ParticleW;
		}
		else if (EntryIndex == INDEX_NONE)
		{
			// not present in current results, just use next results
			const FGeometryCollectionResults::FVelocityData& NextData = NextResults->GetVelocities(NextEntryIndex);
			LinearVelOut = NextData.ParticleV;
			AngularVelOut = NextData.ParticleW;
		}
		else
		{
			// both available 
			const FGeometryCollectionResults::FVelocityData& Data = Results->GetVelocities(EntryIndex);
			const FGeometryCollectionResults::FVelocityData& NextData = NextResults->GetVelocities(NextEntryIndex);
			LinearVelOut = FMath::Lerp(Data.ParticleV, NextData.ParticleV, Alpha);
			AngularVelOut = FMath::Lerp(Data.ParticleW, NextData.ParticleW, Alpha);
		}
	}

private:
	const int32 TransformIndex;
	const FGeometryCollectionResults* Results;
	FGeometryCollectionResults::FEntryIndex EntryIndex;
	const FGeometryCollectionResults* NextResults;
	FGeometryCollectionResults::FEntryIndex NextEntryIndex;
	Chaos::FPBDRigidParticle& GTParticle;
	Chaos::FRealSingle Alpha;

};

bool FGeometryCollectionPhysicsProxy::PullNonInterpolatableDataFromSinglePhysicsState(const Chaos::FDirtyGeometryCollectionData& BufferData, bool bForcePullXRVW, const TBitArray<>* Seen)
{
	const FGeometryCollectionResults& CurrentResults = BufferData.Results();
	const bool bHasResults = CurrentResults.GetNumEntries() > 0;
	if (!bHasResults)
	{
		return false;
	}

	bool bIsCollectionDirty = false;

	TManagedArray<FVector3f>* LinearVelocities = GameThreadCollection.GetLinearVelocitiesAttribute();
	TManagedArray<FVector3f>* AngularVelocities = GameThreadCollection.GetAngularVelocitiesAttribute();
	TManagedArray<uint8>& InternalClusterParentTypeArray = GameThreadCollection.GetInternalClusterParentTypeAttribute();
	const TManagedArray<bool>* AnimationsActive = GameThreadCollection.GetAnimateTransformAttribute();

	// first step : process the non values that do not need to be interpolated 
	for (int32 EntryIndex = 0; EntryIndex < CurrentResults.GetNumEntries(); EntryIndex++)
	{
		const FGeometryCollectionResults::FStateData& StateData = CurrentResults.GetState(EntryIndex);
		const int32 TransformGroupIndex = StateData.TransformIndex;

		// make sure we are not crashing if the transform index is invalid
		const bool bIsValidTransformIndex = FromTransformToParticleIndex.IsValidIndex(TransformGroupIndex);
		ensureMsgf(bIsValidTransformIndex, TEXT(" TransformGroupIndex is not valid : value = [%d] - Max valid TransformIndex = [%d]"), TransformGroupIndex, FromTransformToParticleIndex.Num());
		if (!bIsValidTransformIndex)
		{
			continue;
		}

		if (Seen && (*Seen)[TransformGroupIndex])
		{
			continue;
		}

		const int32 ParticleIndex = FromTransformToParticleIndex[TransformGroupIndex];
		const bool bIsRootIndex = (Parameters.InitialRootIndex == TransformGroupIndex);
		const bool bIsActive = !StateData.State.DisabledState;

		if (!bIsRootIndex && bIsActive)
		{
			CreateChildrenGeometry_External();
		}

		if (UpdateValue(GameThreadCollection.Active[TransformGroupIndex], bIsActive))
		{
			if (ParticleIndex != INDEX_NONE && GTParticles[ParticleIndex].IsValid())
			{
				GTParticles[ParticleIndex]->SetDisabled(!bIsActive);
				bIsCollectionDirty = true;
			}
		}

		if (StateData.State.HasDecayed)
		{
			if (!bIsRootIndex)
			{
				if (ParticleIndex != INDEX_NONE && GTParticles[ParticleIndex].IsValid())
				{
					GTParticles[ParticleIndex]->SetDisabled(true);
					bIsCollectionDirty = true;
				}
			}
		}

		if (ParticleIndex == INDEX_NONE || GTParticles[ParticleIndex] == nullptr)
		{
			continue;
		}

		FParticle& GTParticle = *GTParticles[ParticleIndex];

		if (UpdateValue(GameThreadCollection.DynamicState[TransformGroupIndex], static_cast<uint8>(StateData.State.DynamicState)))
		{
			GTParticle.SetObjectState(static_cast<Chaos::EObjectStateType>(StateData.State.DynamicState));
			bIsCollectionDirty = true;
		}

		if (GameThreadCollection.GetHasParent(TransformGroupIndex) != StateData.HasParent)
		{
			GameThreadCollection.SetHasParent(TransformGroupIndex, StateData.HasParent);
			bIsCollectionDirty = true;
		}

		Chaos::EInternalClusterType ParentType = Chaos::EInternalClusterType::None;
		if (StateData.State.HasInternalClusterParent != 0)
		{
			if (StateData.State.HasClusterUnionParent)
			{
				ParentType = Chaos::EInternalClusterType::ClusterUnion;
			}
			else
			{
				ParentType = (StateData.State.DynamicInternalClusterParent != 0) ? Chaos::EInternalClusterType::Dynamic : Chaos::EInternalClusterType::KinematicOrStatic;
			}
		}
		const uint8 ParentTypeUInt8 = static_cast<uint8>(ParentType);
		if (UpdateValue(InternalClusterParentTypeArray[TransformGroupIndex], ParentTypeUInt8))
		{
			bIsCollectionDirty = true;
		}

		// if interpolation is off , we need to apply XR, VW and transforms
		if (bForcePullXRVW)
		{
			const bool bAnimatingWhileDisabled = AnimationsActive ? (*AnimationsActive)[TransformGroupIndex] : false;
			if (bIsActive || bAnimatingWhileDisabled)
			{
				const FGeometryCollectionResults::FPositionData& PositionData = CurrentResults.GetPositions(EntryIndex);

				const bool XRModified = UpdateGTParticleXR(GTParticle, PositionData.ParticleX, PositionData.ParticleR, Parameters.bForceUpdateActiveTransforms);
				bIsCollectionDirty |= XRModified;

				const FGeometryCollectionResults::FVelocityData& VelocityData = CurrentResults.GetVelocities(EntryIndex);
				const bool VWModified = UpdateGTParticleVW(GTParticle, VelocityData.ParticleV, VelocityData.ParticleW);
				bIsCollectionDirty |= VWModified;

				if (LinearVelocities && AngularVelocities)
				{
					bIsCollectionDirty |= UpdateValue((*LinearVelocities)[TransformGroupIndex], VelocityData.ParticleV);
					bIsCollectionDirty |= UpdateValue((*AngularVelocities)[TransformGroupIndex], VelocityData.ParticleW);
				}
			}
		}

		// internal cluster index map update
		// make sure to update the index even if the InternalClusterUniqueIdx is INDEX_NONE
		GTParticleToIndices.FindOrAdd(GTParticles[ParticleIndex].Get()).InternalClusterUniqueId = StateData.InternalClusterUniqueIdx;
		if (StateData.InternalClusterUniqueIdx > INDEX_NONE)
		{
			InternalClusterUniqueIdxToChildrenTransformIndices.FindOrAdd(StateData.InternalClusterUniqueIdx).Add(TransformGroupIndex);
		}
	}

	// See comments in BufferPhysicsResults_Internal where IsRootBroken is set
	// NOTE: We should never be returning to unbroken once broken although we aren't checking for that...
	const int32 RootIndex = Parameters.InitialRootIndex;
	if ((RootIndex != INDEX_NONE) && CurrentResults.IsRootBroken && GameThreadCollection.Active[RootIndex] && bGeometryCollectionUseRootBrokenFlag)
	{
		GameThreadCollection.Active[RootIndex] = false;
		const int32 ParticleIndex = FromTransformToParticleIndex[RootIndex];
		check(ParticleIndex != INDEX_NONE);
		if (GTParticles[ParticleIndex] != nullptr)
		{
			GTParticles[ParticleIndex]->SetDisabled(true);
			bIsCollectionDirty = true;
		}
	}

	return bIsCollectionDirty;
}

// Called from FPhysScene_ChaosInterface::SyncBodies(), NOT the solver.
bool FGeometryCollectionPhysicsProxy::PullFromPhysicsState(const Chaos::FDirtyGeometryCollectionData& PullData, const int32 SolverSyncTimestamp, const Chaos::FDirtyGeometryCollectionData* NextPullData, const Chaos::FRealSingle* Alpha, const Chaos::FDirtyRigidParticleReplicationErrorData* Error, const Chaos::FReal AsyncFixedTimeStep)
{
	if(IsObjectDeleting) return false;

	/**
	 * CONTEXT: GAMETHREAD (Read Locked)
	 * Perform a similar operation to Sync, but take the data from a gamethread-safe buffer. This will be called
	 * from the game thread when it cannot sync to the physics thread. The simulation is very likely to be running
	 * when this happens so never read any physics thread data here!
	 *
	 * Note: A read lock will have been acquired for this - so the physics thread won't force a buffer flip while this
	 * sync is ongoing
	 *
	 */

	// earlier exit
	const bool bHasResults = (PullData.Results().GetNumEntries() > 0);
	const bool bHasNextResults = (NextPullData && NextPullData->Results().GetNumEntries() > 0);

	if (!bHasResults && !bHasNextResults)
	{
		return false;
	}

	if (NumTransforms != GameThreadCollection.GetNumTransforms())
	{
		// At this point the component may be gone and we have no way to get the name of the asset/component
		UE_LOGF(LogChaos, Error, 
			"Attempt to update physics state on thegamethread where the number of transform (%d) does not match anymore the rest collection (%d) - skipping physics update",
			NumTransforms, 
			GameThreadCollection.GetNumTransforms()
		);
		return false;
	}

	FProxyInterpolationBase* InterpData = GetInterpolationData();
	if (Error)
	{
		InterpData = GetOrCreateErrorInterpolationData<FProxyInterpolationError>();
		check(InterpData);

		Chaos::FReal MaxErrorCorrection = RenderInterpolationCVars::RenderInterpMaximumErrorCorrectionBeforeSnapping;
		Chaos::FReal ErrorCorrectionDuration = RenderInterpolationCVars::RenderInterpErrorCorrectionDuration;
		if (FErrorInterpolationSettings* ErrorSettings = InterpData->GetErrorInterpolationSettings())
		{
			MaxErrorCorrection = ErrorSettings->MaximumErrorCorrectionBeforeSnapping;
			ErrorCorrectionDuration = ErrorSettings->ErrorCorrectionDuration;
		}

		// Accumulate error if within interpolation limits
		if (Error->ErrorX.SizeSquared() < (MaxErrorCorrection * MaxErrorCorrection))
		{
			const int32 RenderInterpErrorCorrectionDurationTicks = FMath::FloorToInt32(ErrorCorrectionDuration / AsyncFixedTimeStep); // Convert duration from seconds to simulation ticks
			InterpData->AccumulateErrorXR(Error->ErrorX, Error->ErrorR, SolverSyncTimestamp, RenderInterpErrorCorrectionDurationTicks);
		}
	}
	const bool bHasInterpolationData = InterpData != nullptr;

	const bool bNeedInterpolation = (NextPullData != nullptr);

	bool bIsCollectionDirty = false;

	if (NumTransforms > 0)
	{
		InternalClusterUniqueIdxToChildrenTransformIndices.Reset();

		// first: non-interpolatable data (everything besides XRVW OR if no next data exists, everything is non-interpolatable).
		if (NextPullData)
		{
			bIsCollectionDirty |= PullNonInterpolatableDataFromSinglePhysicsState(*NextPullData, false, nullptr);
		}
		bIsCollectionDirty |= PullNonInterpolatableDataFromSinglePhysicsState(PullData, !bNeedInterpolation, NextPullData ? &NextPullData->Results().GetModifiedTransformIndices() : nullptr);

		const TManagedArray<bool>* AnimationsActive = GameThreadCollection.GetAnimateTransformAttribute();
		const TManagedArray<FTransform>& MassToLocal = Parameters.RestCollectionShared->GetAttribute<FTransform>(MassToLocalAttributeName, FTransformCollection::TransformGroup);

		// second : interpolate-able ones
		if (bNeedInterpolation)
		{
			const Chaos::FRealSingle LerpAlpha = RenderInterpolationCVars::bSnapToLatest ? 1.0f : *Alpha;

			const FGeometryCollectionResults& PrevResults = PullData.Results();
			const FGeometryCollectionResults& NextResults = NextPullData->Results();

			if (bHasInterpolationData)
			{
				InterpData->UpdateError(SolverSyncTimestamp, AsyncFixedTimeStep);

				if (PrevResults.GetNumEntries() && NextResults.GetNumEntries())
				{
					InterpData->DirectionalDecay(NextResults.GetPositions(0).ParticleX - PrevResults.GetPositions(0).ParticleX, RenderInterpolationCVars::RenderInterpErrorDirectionalDecayMultiplier);
				}
			}

			TManagedArray<FVector3f>* LinearVelocities = GameThreadCollection.GetLinearVelocitiesAttribute();
			TManagedArray<FVector3f>* AngularVelocities = GameThreadCollection.GetAngularVelocitiesAttribute();

			bool bAtLeatOneChildActive = false;

			// for that case we cannot just go through the list of entries since Results and NextResults may have different number of entries that don't always match
			// so we need to go through the transform indices and find the matching entries on both side 
			// this is helped by the FResultAccessor structure
			for (int32 ParticleIndex = 0; ParticleIndex < NumEffectiveParticles; ++ParticleIndex)
			{
				const int32 TransformGroupIndex = FromParticleToTransformIndex[ParticleIndex];
				const bool bActive = GameThreadCollection.Active[TransformGroupIndex];
				bAtLeatOneChildActive |= bActive && TransformGroupIndex != Parameters.InitialRootIndex;
				if (bAtLeatOneChildActive)
				{
					CreateChildrenGeometry_External();
				}

				if (GTParticles[ParticleIndex] == nullptr)
				{
					continue;
				}

				FParticle& GTParticle = *GTParticles[ParticleIndex];
				const FResultInterpolator ResultInterpolator(TransformGroupIndex, PrevResults, NextResults, GTParticle, LerpAlpha);
				if (ResultInterpolator.HasNoEntry())
				{
					continue;
				}

				const bool bAnimatingWhileDisabled = AnimationsActive ? (*AnimationsActive)[TransformGroupIndex] : false;

				if (bActive || bAnimatingWhileDisabled)
				{
					Chaos::FVec3 NewX;
					Chaos::FRotation3 NewR;
					ResultInterpolator.GetPositions(NewX, NewR);

					const bool bIsReplicationErrorSmoothing = bHasInterpolationData ? InterpData->IsErrorSmoothing() : false;

					if (bIsReplicationErrorSmoothing)
					{
						NewX = NewX + InterpData->GetErrorX(LerpAlpha);
						NewR = NewR * InterpData->GetErrorR(LerpAlpha); // Add rotational error offset in local space
					}

#if CHAOS_DEBUG_DRAW
					if (RenderInterpolationCVars::bRenderInterpDebugDrawGC)
					{
						FGeometryCollectionResults::FEntryIndex EntryIndex = PrevResults.GetEntryIndexByTransformIndex(TransformGroupIndex);
						FGeometryCollectionResults::FEntryIndex NextEntryIndex = NextResults.GetEntryIndexByTransformIndex(TransformGroupIndex);

						if (EntryIndex != INDEX_NONE && NextEntryIndex != INDEX_NONE)
						{
							const FGeometryCollectionResults::FPositionData& PullDataT = PrevResults.GetPositions(EntryIndex);
							const FGeometryCollectionResults::FPositionData& NextPullDataT = NextResults.GetPositions(NextEntryIndex);

							bool DirectionalDecayPerformed = false;
							const FVector ZOffset = FVector(0, 0, RenderInterpolationCVars::RenderInterpDebugDrawZOffset);
							Chaos::FDebugDrawQueue::GetInstance().DrawDebugBox(ZOffset + NextPullDataT.ParticleX, FVector(2, 1, 1), NextPullDataT.ParticleR, FColor::Yellow, false, 3.0f, 0, 0.5f);
							Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow(ZOffset + PullDataT.ParticleX, ZOffset + NextPullDataT.ParticleX, 0.5f, FColor::Yellow, false, 3.0f, 0, 0.5f);
							Chaos::FDebugDrawQueue::GetInstance().DrawDebugBox(ZOffset + NewX, FVector(2, 1, 1), NewR, DirectionalDecayPerformed ? FColor::Cyan : FColor::Green, false, 3.0f, 0, 0.5f);

							if (bIsReplicationErrorSmoothing)
							{
								if (Error)
								{
									Chaos::FDebugDrawQueue::GetInstance().DrawDebugBox(ZOffset + PullDataT.ParticleX, FVector(4, 2, 2), PullDataT.ParticleR, FColor::Red, false, 3.0f, 0, 0.5f);
									Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow(ZOffset + PullDataT.ParticleX + Error->ErrorX, ZOffset + PullDataT.ParticleX, 1, FColor::Red, false, 3.0f, 0, 0.5f);
								}

								Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow(ZOffset + (NewX - InterpData->GetErrorX(LerpAlpha)), ZOffset + NewX, 1, FColor::Blue, false, 3.0f, 0, 0.5f);
							}
						}
					}
#endif // CHAOS_DEBUG_DRAW



					const bool XRModified = UpdateGTParticleXR(GTParticle, NewX, NewR, Parameters.bForceUpdateActiveTransforms);
					bIsCollectionDirty |= XRModified;

					Chaos::FVec3f NewV;
					Chaos::FVec3f NewW;
					ResultInterpolator.GetVelocities(NewV, NewW);
					const bool VWModified = UpdateGTParticleVW(GTParticle, NewV, NewW);
					bIsCollectionDirty |= VWModified;

					if (LinearVelocities && AngularVelocities)
					{
						bIsCollectionDirty |= UpdateValue((*LinearVelocities)[TransformGroupIndex], NewV);
						bIsCollectionDirty |= UpdateValue((*AngularVelocities)[TransformGroupIndex], NewW);
					}
				}
			}
		}

		// update the parent transform now that we have all X and R computed
		bool bHasDifferentTransforms = false;
		if (bIsCollectionDirty)
		{
			const TBitArray<>& PrevResultsModifiedIndices = PullData.Results().GetModifiedTransformIndices();

			const bool bIsComponentTransformScaled = !WorldTransform_External.GetScale3D().Equals(FVector::OneVector);
			const FTransform ComponentScaleTransform(FQuat::Identity, FVector::ZeroVector, WorldTransform_External.GetScale3D());

			for (int32 ParticleIndex = 0; ParticleIndex < NumEffectiveParticles; ParticleIndex++)
			{
				const int32 TransformIndex = FromParticleToTransformIndex[ParticleIndex];
				const bool bAnimatingWhileDisabled = AnimationsActive ? (*AnimationsActive)[TransformIndex] : false;
				const bool bActive = GameThreadCollection.Active[TransformIndex];
				if (bActive || bAnimatingWhileDisabled)
				{
					if (GTParticles[ParticleIndex] != nullptr)
					{
						bool bWasModified = PrevResultsModifiedIndices.IsValidIndex(TransformIndex) ? PrevResultsModifiedIndices[TransformIndex] : false;
						if (NextPullData)
						{
							const TBitArray<>& NextResultsModifiedIndices = NextPullData->Results().GetModifiedTransformIndices();
							bWasModified |= NextResultsModifiedIndices.IsValidIndex(TransformIndex) ? NextResultsModifiedIndices[TransformIndex] : false;
						}
						if (bWasModified)
						{
							if (RebaseParticleGameThreadCollectionTransformOnNewWorldTransform_External(ParticleIndex, MassToLocal, bIsComponentTransformScaled, ComponentScaleTransform))
							{
								bHasDifferentTransforms = true;
							}
						}
					}
				}
			}
		}

#if WITH_EDITORONLY_DATA
		// Damage is collected in full every time so no need to go through PullNonInterpolatableDataFromSinglePhysicsState.
		if (FDamageCollector* Collector = FRuntimeDataCollector::GetInstance().Find(CollectorGuid))
		{
			const FGeometryCollectionResults& CurrentResults = NextPullData ? NextPullData->Results() : PullData.Results();
			for (int32 TransformGroupIndex = 0; TransformGroupIndex < NumTransforms; ++TransformGroupIndex)
			{
				const FGeometryCollectionResults::FDamageData& DamageData = CurrentResults.GetDamages(TransformGroupIndex);
				Collector->SampleDamage(TransformGroupIndex, DamageData.Damage, DamageData.DamageThreshold);
			}
		}
#endif

		// if physics world transform was dirtied this frame we need to force an update to make sure transforms of 
		// broken sleeping particles on kinemically driven GCs remained updated and the object feel grounded to the world

		GameThreadCollection.MakeClean();
		if (bIsCollectionDirty || bIsGameThreadWorldTransformDirty)
		{
			GameThreadCollection.MakeDirty();
		}
		bIsGameThreadWorldTransformDirty = false;
	}

	if (PostPhysicsSyncCallback)
	{
		PostPhysicsSyncCallback();
	}
	
	return true;
}

void FGeometryCollectionPhysicsProxy::UpdateFilterData_External(const FCollisionFilterData& NewSimFilter, const FCollisionFilterData& NewQueryFilter)
{
	// SimFilter/QueryFilter members are read on both threads, these are const after initialization and are not updated here.
	Chaos::Filter::FCombinedShapeFilterData NewShapeFilter = Chaos::Filter::FShapeFilterBuilder::BuildFromLegacyShapeFilter(NewQueryFilter, NewSimFilter);
	UpdateFilterData_External(NewShapeFilter);
}

void FGeometryCollectionPhysicsProxy::UpdateFilterData_External(const Chaos::Filter::FCombinedShapeFilterData& NewShapeFilter)
{
	SCOPE_CYCLE_COUNTER(STAT_GCUpdateFilterData);
	check(IsInGameThread());

	// ShapeFilter members are read on both threads, these are const after initialization and are not updated here.
	for (int32 Index = 0; Index < NumEffectiveParticles; ++Index)
	{
		FParticle* P = GTParticles[Index].Get();
		if (P != nullptr)
		{
			const Chaos::FShapesArray& Shapes = P->ShapesArray();
			const int32 NumShapes = Shapes.Num();
			for (int32 ShapeIndex = 0; ShapeIndex < NumShapes; ++ShapeIndex)
			{
				Chaos::FPerShapeData* Shape = Shapes[ShapeIndex].Get();
				Shape->SetCombinedShapeFilterData(NewShapeFilter);
			}
		}
	}

	ExecuteOnPhysicsThread(*this, [this, NewShapeFilter]()
		{
			SetFilterData_Internal(NewShapeFilter);
		});
}

void FGeometryCollectionPhysicsProxy::SetFilterData_Internal(const FCollisionFilterData& NewSimFilter, const FCollisionFilterData& NewQueryFilter)
{
	Chaos::Filter::FCombinedShapeFilterData NewShapeFilter = Chaos::Filter::FShapeFilterBuilder::BuildFromLegacyShapeFilter(NewQueryFilter, NewSimFilter);
	SetFilterData_Internal(NewShapeFilter);
}

void FGeometryCollectionPhysicsProxy::SetFilterData_Internal(const Chaos::Filter::FCombinedShapeFilterData& NewShapeFilter)
{
	using namespace Chaos;

	Parameters.ShapeFilterData = NewShapeFilter.GetShapeFilterData();
	Parameters.ShapeInstanceData = NewShapeFilter.GetInstanceData();

	TArray<FClusterHandle*> ProcessedInternalClusters;
	if (Chaos::FPhysicsSolver* RigidSolver = GetSolver<Chaos::FPhysicsSolver>())
	{
		FRigidClustering& RigidClustering = RigidSolver->GetEvolution()->GetRigidClustering();

		for (int32 ParticleIndex = 0; ParticleIndex < NumEffectiveParticles; ++ParticleIndex)
		{
			if (Chaos::FPBDRigidClusteredParticleHandle* Handle = SolverParticleHandles[ParticleIndex])
			{
				// Must update our filters before updating an internal cluster parent
				const Chaos::FShapesArray& ShapesArray = Handle->ShapesArray();
				for (const TUniquePtr<Chaos::FPerShapeData>& Shape : ShapesArray)
				{
					Shape->SetCombinedShapeFilterData(NewShapeFilter);
				}

				FClusterHandle* ParentHandle = Handle->Parent();
				if (ParentHandle && ParentHandle->InternalCluster() && ParentHandle->PhysicsProxy() == this)
				{
					if (!ProcessedInternalClusters.Contains(ParentHandle))
					{
						ProcessedInternalClusters.Add(ParentHandle);
						// Must update our filters before updating an internal cluster parent
						FRigidClustering::FRigidHandleArray* ChildArray = RigidClustering.GetChildrenMap().Find(ParentHandle);
						if (ensure(ChildArray))
						{
							// If our filter changed, internal cluster parent's filter may be stale.
							UpdateClusterFilterDataFromChildren(ParentHandle, *ChildArray);
						}
					}
				}
			}
		}
	}
}

DECLARE_CYCLE_STAT(TEXT("FGeometryCollectionPhysicsProxy::UpdatePerParticleFilterData_External"), STAT_GeometryCollectionPhysicsProxyUpdatePerParticleFilterData, STATGROUP_Chaos);
void FGeometryCollectionPhysicsProxy::UpdatePerParticleFilterData_External(const TArray<FParticleCollisionFilterData>& PerParticleData)
{
	SCOPE_CYCLE_COUNTER(STAT_GeometryCollectionPhysicsProxyUpdatePerParticleFilterData);

	// TODO: Need to figure out how the per-particle collision filter data works with the global one. The per-particle one should probably replace the global one entirely...
	ExecuteOnPhysicsThread(*this, [this, PerParticleData]()
		{
			SetPerParticleFilterData_Internal(PerParticleData);
		});
}

void FGeometryCollectionPhysicsProxy::SetPerParticleFilterData_Internal(const TArray<FParticleCollisionFilterData>& PerParticleData)
{
	if (Chaos::FPhysicsSolver* RBDSolver = GetSolver<Chaos::FPhysicsSolver>())
	{
		Chaos::FWritePhysicsObjectInterface_Internal Interface = Chaos::FPhysicsObjectInternalInterface::GetWrite();

		for (const FParticleCollisionFilterData& Data : PerParticleData)
		{
			if (Data.bIsValid && Data.ParticleIndex != INDEX_NONE)
			{
				const int32 ParticleIndex = FromTransformToParticleIndex[Data.ParticleIndex];
				if (ParticleIndex != INDEX_NONE)
				{
					Chaos::FPhysicsObjectHandle Object = PhysicsObjects[ParticleIndex].Get();
					check(Object != nullptr);
					TArrayView<Chaos::FPhysicsObjectHandle> ParticleView{ &Object, 1 };
					Interface.UpdateShapeCollisionFlags(ParticleView, Data.bSimEnabled, Data.bQueryEnabled);
					Interface.UpdateCombinedShapeFilterData(ParticleView, Chaos::Filter::FCombinedShapeFilterData(Data.ShapeFilterData, Data.FilterInstanceData));
					// todo(chaos): It's not ideal but the geometry collection needs to request the cluster union update its
					// cached shape data if the particle is in a cluster union. This is because we don't share shape
					// data between the GC shapes and the cluster union shapes.
					Chaos::FRigidClustering& Clustering = RBDSolver->GetEvolution()->GetRigidClustering();
					Chaos::FClusterUnionManager& ClusterUnionManager = Clustering.GetClusterUnionManager();

					if (Chaos::FPBDRigidClusteredParticleHandle* Particle = SolverParticleHandles[ParticleIndex])
					{
						if (Chaos::FClusterUnion* ClusterUnion = ClusterUnionManager.FindClusterUnionFromParticle(Particle))
						{
							ClusterUnion->AddPendingGeometryOperation(Chaos::EClusterUnionGeometryOperation::Refresh, Particle);
							ClusterUnionManager.RequestDeferredClusterPropertiesUpdate(ClusterUnion->InternalIndex, Chaos::EUpdateClusterUnionPropertiesFlags::IncrementalGenerateGeometry);
						}
					}
				}
			}
		}
	}
}

//==============================================================================
// STATIC SETUP FUNCTIONS
//==============================================================================
static const FSharedSimulationSizeSpecificData& GetSizeSpecificData(const TArray<FSharedSimulationSizeSpecificData>& SizeSpecificData, const FGeometryCollection& RestCollection, const int32 TransformIndex, const FBox& BoundingBox)
{
	// If we have a normalized Size available, use that to determine SizeSpecific index, otherwise fall back on Bounds volume.
	int32 SizeSpecificIdx;
	const TManagedArray<float>* RelativeSizes = RestCollection.FindAttribute<float>("Size", FTransformCollection::TransformGroup);
	if (RelativeSizes)
	{
		SizeSpecificIdx = GeometryCollection::SizeSpecific::FindIndexForVolume(SizeSpecificData, (*RelativeSizes)[TransformIndex]);
	}
	else
	{
		SizeSpecificIdx = GeometryCollection::SizeSpecific::FindIndexForVolume(SizeSpecificData, BoundingBox);
	}
	return SizeSpecificData[SizeSpecificIdx];
}

// compute inner and outer radii from geometry in parallel if the attribute value as not been set
static void GenerateInnerAndOuterRadiiIfNeeded(FGeometryCollection& RestCollection)
{
	const int32 NumGeometries = RestCollection.NumElements(FGeometryCollection::GeometryGroup);
	const TManagedArray<FVector3f>& Vertices = RestCollection.Vertex;

	ParallelFor(NumGeometries, [&](int32 GeometryIndex)
	{
		if (RestCollection.InnerRadius[GeometryIndex] == 0.0f || RestCollection.OuterRadius[GeometryIndex] == 0.0f)
		{
			GeometryCollection::ComputeInnerAndOuterRadiiFromGeometryVertices(
				Vertices,
				RestCollection.VertexStart[GeometryIndex],
				RestCollection.VertexCount[GeometryIndex],
				RestCollection.InnerRadius[GeometryIndex],
				RestCollection.OuterRadius[GeometryIndex]);
		}
	});
}

static Chaos::FImplicitObjectPtr CreateImplicitGeometry(
	const FSharedSimulationSizeSpecificData& SizeSpecificData,
	const int32 TransformGroupIndex,
	const FGeometryCollection& RestCollection,
	const Chaos::FParticles& MassSpaceParticles,
	const Chaos::FTriangleMesh& TriMesh,
	const FBox& InstanceBoundingBox,
	const Chaos::FReal InnerRadius,
	const Chaos::FErrorReporter& ErrorReporter,
	const Chaos::FVec3* ClusterMaxChildBounds = nullptr
	)
{
	Chaos::FImplicitObjectPtr NewImplicit;
	if (SizeSpecificData.CollisionShapesData.Num())
	{
		const TManagedArray<FTransform>& CollectionMassToLocal = RestCollection.GetAttribute<FTransform>(MassToLocalAttributeName, FTransformCollection::TransformGroup);
		const TManagedArray<TSet<int32>>* TransformToConvexIndices = RestCollection.FindAttribute<TSet<int32>>("TransformToConvexIndices", FTransformCollection::TransformGroup);
		const TManagedArray<Chaos::FConvexPtr>* ConvexGeometry = RestCollection.FindAttribute<Chaos::FConvexPtr>(FGeometryCollection::ConvexHullAttribute, FGeometryCollection::ConvexGroup);

		const FCollectionCollisionTypeData& CollisionTypeData = SizeSpecificData.CollisionShapesData[0]; 
		switch(CollisionTypeData.ImplicitType)
		{
		case EImplicitTypeEnum::Chaos_Implicit_LevelSet:
			{
				const FCollectionLevelSetData& LevelSetData = CollisionTypeData.LevelSetData; 
				int32 MinResolution = LevelSetData.MinLevelSetResolution;
				int32 MaxResolution = LevelSetData.MaxLevelSetResolution;
				if (ClusterMaxChildBounds)
				{
					const Chaos::FVec3 Scale = 2. * InstanceBoundingBox.GetExtent() / (*ClusterMaxChildBounds); // FBox's extents are 1/2 (Max - Min)
					const Chaos::FReal ScaleMax = Scale.GetAbsMax();
					const Chaos::FReal ScaleMin = Scale.GetAbsMin();

					Chaos::FReal MinResolutionReal = ScaleMin * (Chaos::FReal)LevelSetData.MinLevelSetResolution;
					MinResolutionReal = FMath::Clamp(MinResolutionReal,
						(Chaos::FReal)LevelSetData.MinLevelSetResolution,
						(Chaos::FReal)LevelSetData.MinClusterLevelSetResolution);

					Chaos::FReal MaxResolutionReal = ScaleMax * (Chaos::FReal)LevelSetData.MaxLevelSetResolution;
					MaxResolutionReal = FMath::Clamp(MaxResolutionReal,
						(Chaos::FReal)LevelSetData.MaxLevelSetResolution,
						(Chaos::FReal)LevelSetData.MaxClusterLevelSetResolution);

					MinResolution = FMath::FloorToInt32(MinResolutionReal);
					MaxResolution = FMath::FloorToInt32(MaxResolutionReal);
				}
				
				NewImplicit = Chaos::FImplicitObjectPtr(
					FCollisionStructureManager::NewImplicitLevelset(
						ErrorReporter,
						MassSpaceParticles,
						TriMesh,
						InstanceBoundingBox,
						MinResolution,
						MaxResolution,
						CollisionTypeData.CollisionObjectReductionPercentage,
						CollisionTypeData.CollisionType));
				
				// Fall back on sphere if level set rasterization failed.
				if (!NewImplicit)
				{
					NewImplicit = Chaos::FImplicitObjectPtr(
						FCollisionStructureManager::NewImplicitSphere(
						InnerRadius,
						CollisionTypeData.CollisionObjectReductionPercentage,
						CollisionTypeData.CollisionType));
				}
			}
			break;
				
		case EImplicitTypeEnum::Chaos_Implicit_Box:
			{
				NewImplicit = Chaos::FImplicitObjectPtr(
					FCollisionStructureManager::NewImplicitBox(
						InstanceBoundingBox,
						CollisionTypeData.CollisionObjectReductionPercentage,
						CollisionTypeData.CollisionType));
			}
			break;
			
		case EImplicitTypeEnum::Chaos_Implicit_Sphere:
			{
				NewImplicit = Chaos::FImplicitObjectPtr(
					FCollisionStructureManager::NewImplicitSphere(
						InnerRadius,
						CollisionTypeData.CollisionObjectReductionPercentage,
						CollisionTypeData.CollisionType));
			}
			break;
			
		case EImplicitTypeEnum::Chaos_Implicit_Convex:
			{
				if (ConvexGeometry && TransformToConvexIndices)
				{
					NewImplicit = Chaos::FImplicitObjectPtr(
						FCollisionStructureManager::NewImplicitConvex(
							(*TransformToConvexIndices)[TransformGroupIndex].Array(),
							ConvexGeometry,
							CollisionTypeData.CollisionType,
							CollectionMassToLocal[TransformGroupIndex],
							CollisionTypeData.CollisionMarginFraction,
							CollisionTypeData.CollisionObjectReductionPercentage
						)
					);
				}
			}
			break;
			
		case EImplicitTypeEnum::Chaos_Implicit_Capsule:
			{
				NewImplicit = Chaos::FImplicitObjectPtr(
					FCollisionStructureManager::NewImplicitCapsule(
						InstanceBoundingBox,
						CollisionTypeData.CollisionObjectReductionPercentage,
						CollisionTypeData.CollisionType));
			}
			break;
		
		case EImplicitTypeEnum::Chaos_Implicit_None:
			// nothing to do here 
			break;
			
		default:
			ensure(false); // unsupported implicit type!
			break;
		}
	}
	return NewImplicit;
}

static void ComputeGeometryVolumeAndCenterOfMass(
	const TArray<FVector3f>& Vertices,
	const Chaos::FTriangleMesh& TriMesh,
	const FBox& BoundingBox,
	const FSharedSimulationParameters& SharedParams, 
	Chaos::FReal& OutUncorrectedVolume,
	Chaos::FReal& OutVolume,
	Chaos::FVec3& OutCenterOfMass,
	bool& OutIsTooSmallGeometry
	)
{
	const float MinVolume = FMath::Max(UE_SMALL_NUMBER, SharedParams.MinimumVolumeClamp());
	const float MaxVolume = FMath::Max(MinVolume, SharedParams.MaximumVolumeClamp());

	OutUncorrectedVolume = 0.f;
	OutVolume = 0.f;
    OutCenterOfMass = FVector::ZeroVector;
	OutIsTooSmallGeometry = (BoundingBox.GetExtent().GetAbsMin() < MinVolume);
	
	if (!OutIsTooSmallGeometry)
	{
		float Volume;
		FVector3f CenterOfMass;
		Chaos::CalculateVolumeAndCenterOfMass(Vertices, TriMesh.GetElements(), Volume, CenterOfMass);
		OutVolume = Volume;
		OutUncorrectedVolume = Volume;
		OutCenterOfMass = CenterOfMass;
		
		if (OutUncorrectedVolume == 0)
		{
			Chaos::CalculateVolumeAndCenterOfMass(BoundingBox, OutVolume, OutCenterOfMass);
		}
		if (OutUncorrectedVolume < MinVolume)
		{
			// For rigid bodies outside of range just default to a clamped bounding box, and warn the user.
			OutVolume = MinVolume;
			OutCenterOfMass = BoundingBox.GetCenter();
		}
		else if (MaxVolume < OutUncorrectedVolume)
		{
			// For rigid bodies outside of range just default to a clamped bounding box, and warn the user
			OutVolume = MaxVolume;
			OutCenterOfMass = BoundingBox.GetCenter();
		}
	}
}

static void ComputeMassPropertyFromMesh(
	const int32 GeometryIndex,
	const FGeometryCollection& RestCollection,
	const FSharedSimulationParameters& SharedParams,
	TUniquePtr<Chaos::FTriangleMesh>& TriMesh,
	Chaos::FMassProperties& OutMassProperties,
	bool& bOutIsTooSmallGeometry
)
{
	// VerticesGroup
	const TManagedArray<FVector3f>& Vertex = RestCollection.Vertex;

	// FacesGroup
	const TManagedArray<bool>& Visible = RestCollection.Visible;
	const TManagedArray<FIntVector>& Indices = RestCollection.Indices;

	// GeometryGroup
	const TManagedArray<FBox>& BoundingBox = RestCollection.BoundingBox;
	const TManagedArray<int32>& FaceStart = RestCollection.FaceStart;
	const TManagedArray<int32>& FaceCount = RestCollection.FaceCount;

	
	TriMesh = CreateTriangleMesh(
		FaceStart[GeometryIndex],
		FaceCount[GeometryIndex],
		Visible,
		Indices);

	Chaos::FReal UncorrectedVolume = 0;
	
	bOutIsTooSmallGeometry = false;
	ComputeGeometryVolumeAndCenterOfMass(
		Vertex.GetConstArray(),
		*TriMesh,
		BoundingBox[GeometryIndex],
		SharedParams,
		UncorrectedVolume,
		OutMassProperties.Volume,
		OutMassProperties.CenterOfMass,
		bOutIsTooSmallGeometry
		);

	// now compute unit inertia ( assuming density of 1 )
	constexpr Chaos::FReal UnitDensityKGPerCM = 1;
	const bool bUseBoundingBoxInertia = (UncorrectedVolume == 0);  
	if (bUseBoundingBoxInertia)
	{
		OutMassProperties.CenterOfMass = BoundingBox[GeometryIndex].GetCenter();
		Chaos::CalculateInertiaAndRotationOfMass(BoundingBox[GeometryIndex], UnitDensityKGPerCM, OutMassProperties.InertiaTensor, OutMassProperties.RotationOfMass);
	}
	else
	{
		const FVector3f CenterOfMass{ OutMassProperties.CenterOfMass };
		Chaos::PMatrix<float,3,3> InertiaTensor;
		Chaos::TRotation<float,3> RotationOfMass;
		CalculateInertiaAndRotationOfMass(Vertex.GetConstArray(), TriMesh->GetSurfaceElements(), (float)UnitDensityKGPerCM, CenterOfMass, InertiaTensor, RotationOfMass);
		OutMassProperties.CenterOfMass = CenterOfMass;
		OutMassProperties.InertiaTensor = InertiaTensor;
		OutMassProperties.RotationOfMass = RotationOfMass;

		// scale the inertia tensor accordingly to compensate for the volume adjustment
		if (OutMassProperties.Volume != UncorrectedVolume)
		{
			const FVector::FReal InertiaScale = OutMassProperties.Volume / UncorrectedVolume;
			OutMassProperties.InertiaTensor = OutMassProperties.InertiaTensor.ApplyScale(InertiaScale);
		}
	}
}

// compute mass properties and trimesh in parallel
// ( non-simulatable particles will not compute the it  ) 
// also return if the geometry mesh is too small 
static void ComputeMassPropertiesAndTriMeshes(
	const FGeometryCollection& RestCollection,
	const FSharedSimulationParameters& SharedParams,
	TArray<TUniquePtr<Chaos::FTriangleMesh>>& OutTriangleMeshesArray,
	TArray<Chaos::FMassProperties>& MassPropertiesArray,
	TArray<bool>& IsTooSmallGeometryArray
	)
{
	const int32 NumGeometries = RestCollection.NumElements(FGeometryCollection::GeometryGroup);
	const TManagedArray<int32>& TransformIndex = RestCollection.TransformIndex;
	const TManagedArray<bool>& CollectionSimulatableParticles = RestCollection.GetAttribute<bool>(FGeometryCollection::SimulatableParticlesAttribute, FTransformCollection::TransformGroup);
	const TManagedArray<Chaos::FImplicitObjectPtr>* ExternaCollisions = RestCollection.FindAttribute<Chaos::FImplicitObjectPtr>(FGeometryCollection::ExternalCollisionsAttribute, FGeometryCollection::TransformGroup);
	const TManagedArray<int32>& SimulationType = RestCollection.SimulationType;

	ParallelFor(NumGeometries, [&](int32 GeometryIndex)
	{
		const int32 TransformGroupIndex = TransformIndex[GeometryIndex];
		if (CollectionSimulatableParticles[TransformGroupIndex])
		{
			// todo compute mesh here and pass it on 
			ComputeMassPropertyFromMesh(GeometryIndex, RestCollection, SharedParams, OutTriangleMeshesArray[GeometryIndex], MassPropertiesArray[GeometryIndex], IsTooSmallGeometryArray[GeometryIndex]);
			// todo : needs to be a better decision as we are still computing the inertia above to replace it ?( see comment above )
			if (SharedParams.bUseImportedCollisionImplicits && ExternaCollisions && (*ExternaCollisions)[TransformGroupIndex])
			{
				constexpr Chaos::FReal UnitDensityKGPerCM = 1;
				Chaos::CalculateMassPropertiesOfImplicitType(MassPropertiesArray[GeometryIndex], Chaos::FRigidTransform3::Identity, (*ExternaCollisions)[TransformGroupIndex].GetReference(), UnitDensityKGPerCM);
				IsTooSmallGeometryArray[GeometryIndex] = false;
			}
		}

	});
}

static Chaos::FImplicitObjectPtr MakeTransformImplicitObject(const Chaos::FImplicitObject& ImplicitObject, const Chaos::FRigidTransform3& Transform)
{
	Chaos::FImplicitObjectPtr ResultObject;
	
	// we cannot really put a transform on top a union, so we need to transform each member
	if (ImplicitObject.IsUnderlyingUnion())
	{
		TArray<Chaos::FImplicitObjectPtr> TransformedObjects;
		const Chaos::FImplicitObjectUnion& Union = static_cast<const Chaos::FImplicitObjectUnion&>(ImplicitObject);
		for (const Chaos::FImplicitObjectPtr& Object: Union.GetObjects())
		{
			TransformedObjects.Add(MakeTransformImplicitObject(*Object, Transform));
		}
		ResultObject = MakeImplicitObjectPtr<Chaos::FImplicitObjectUnion>(MoveTemp(TransformedObjects));
	}
	else if (ImplicitObject.GetType() == Chaos::ImplicitObjectType::Transformed)
	{
		const Chaos::TImplicitObjectTransformed<Chaos::FReal,3>& TransformObject = static_cast<const Chaos::TImplicitObjectTransformed<Chaos::FReal,3>&>(ImplicitObject);
		// we deep copy at this point as the transform is going to be handled at this level
		Chaos::FImplicitObjectPtr TransformedObjectCopy =  TransformObject.GetTransformedObject()->DeepCopyGeometry();
		ResultObject = MakeImplicitObjectPtr<Chaos::TImplicitObjectTransformed<Chaos::FReal,3>>(MoveTemp(TransformedObjectCopy),  TransformObject.GetTransform() * Transform);
	}
	else
	{
		// we deep copy at this point as the transform is going to be handled at this level
		Chaos::FImplicitObjectPtr TransformedObjectCopy =  ImplicitObject.DeepCopyGeometry();
		ResultObject = MakeImplicitObjectPtr<Chaos::TImplicitObjectTransformed<Chaos::FReal,3>>(MoveTemp(TransformedObjectCopy), Transform);
	}
	return ResultObject;
}

bool FGeometryCollectionPhysicsProxy::NeedToInitializeSharedCollisionStructures(const FGeometryCollection& RestCollection)
{
	return !RestCollection.HasAttribute(InertiaTensorAttributeName, FTransformCollection::TransformGroup)
		|| !RestCollection.HasAttribute(MassAttributeName, FTransformCollection::TransformGroup)
		|| !RestCollection.HasAttribute(FGeometryDynamicCollection::SimplicialsAttribute, FTransformCollection::TransformGroup)
		|| !RestCollection.HasAttribute(LevelAttributeName, FTransformCollection::TransformGroup)
		|| !RestCollection.HasAttribute(FGeometryDynamicCollection::ImplicitsAttribute, FTransformCollection::TransformGroup)
		|| !RestCollection.HasAttribute(MassToLocalAttributeName, FTransformCollection::TransformGroup)
		;
}

/** 
	NOTE - Making any changes to data stored on the rest collection below MUST be accompanied
	by a rotation of the DDC key in FDerivedDataGeometryCollectionCooker::GetVersionString
*/
void FGeometryCollectionPhysicsProxy::InitializeSharedCollisionStructures(
	Chaos::FErrorReporter& ErrorReporter,
	FGeometryCollection& RestCollection,
	const FSharedSimulationParameters& SharedParams)
{
	check(SharedParams.SizeSpecificData.Num());

	FString BaseErrorPrefix = ErrorReporter.GetPrefix();

	// fracture tools can create an empty GC before appending new geometry
	if (RestCollection.NumElements(FGeometryCollection::GeometryGroup) == 0)
	{
		return;
	}

	using namespace Chaos;

	// TransformGroup
	const TManagedArray<int32>& BoneMap = RestCollection.BoneMap;
	const TManagedArray<int32>& Parent = RestCollection.Parent;
	const TManagedArray<TSet<int32>>& Children = RestCollection.Children;
	const TManagedArray<int32>& SimulationType = RestCollection.SimulationType;
	TManagedArray<bool>& CollectionSimulatableParticles = RestCollection.ModifyAttribute<bool>(FGeometryCollection::SimulatableParticlesAttribute, FTransformCollection::TransformGroup);
	TManagedArray<FVector3f>& CollectionInertiaTensor = RestCollection.AddAttribute<FVector3f>(InertiaTensorAttributeName, FTransformCollection::TransformGroup);
	TManagedArray<FRealSingle>& CollectionMass = RestCollection.AddAttribute<FRealSingle>(MassAttributeName, FTransformCollection::TransformGroup);
	TManagedArray<TUniquePtr<FSimplicial>>& CollectionSimplicials =	RestCollection.AddAttribute<TUniquePtr<FSimplicial>>(FGeometryDynamicCollection::SimplicialsAttribute, FTransformCollection::TransformGroup);

	TManagedArray<int32>& Levels = RestCollection.AddAttribute<int32>(LevelAttributeName, FTransformCollection::TransformGroup);

	RestCollection.RemoveAttribute(FGeometryDynamicCollection::ImplicitsAttribute, FTransformCollection::TransformGroup);
	TManagedArray<Chaos::FImplicitObjectPtr>& CollectionImplicits = RestCollection.AddAttribute<Chaos::FImplicitObjectPtr>(FGeometryDynamicCollection::ImplicitsAttribute, FTransformCollection::TransformGroup);

	bool bUseRelativeSize = RestCollection.HasAttribute(TEXT("Size"), FTransformCollection::TransformGroup);
	if (!bUseRelativeSize)
	{
		UE_LOGF(LogChaos, Display, "Relative Size not found on Rest Collection. Using bounds volume for SizeSpecificData indexing instead.");
	}


	// @todo(chaos_transforms) : do we still use this?
	TManagedArray<FTransform>& CollectionMassToLocal = RestCollection.AddAttribute<FTransform>(MassToLocalAttributeName, FTransformCollection::TransformGroup);
	FTransform IdentityXf(FQuat::Identity, FVector(0));
	IdentityXf.NormalizeRotation();
	CollectionMassToLocal.Fill(IdentityXf);

	// VerticesGroup
	const TManagedArray<FVector3f>& Vertex = RestCollection.Vertex;

	// FacesGroup
	const TManagedArray<bool>& Visible = RestCollection.Visible;
	const TManagedArray<FIntVector>& Indices = RestCollection.Indices;

	// GeometryGroup
	const TManagedArray<int32>& TransformIndex = RestCollection.TransformIndex;
	const TManagedArray<FBox>& BoundingBox = RestCollection.BoundingBox;
	TManagedArray<Chaos::FRealSingle>& InnerRadius = RestCollection.InnerRadius;
	TManagedArray<Chaos::FRealSingle>& OuterRadius = RestCollection.OuterRadius;
	const TManagedArray<int32>& VertexStart = RestCollection.VertexStart;
	const TManagedArray<int32>& VertexCount = RestCollection.VertexCount;
	const TManagedArray<int32>& FaceStart = RestCollection.FaceStart;
	const TManagedArray<int32>& FaceCount = RestCollection.FaceCount;
	
	TArray<FTransform> CollectionSpaceTransforms;
	{ // tmp scope
		const TManagedArray<FTransform3f>& HierarchyTransform = RestCollection.Transform;
		GeometryCollectionAlgo::GlobalMatrices(HierarchyTransform, Parent, CollectionSpaceTransforms);
	} // tmp scope

	const int32 NumTransforms = CollectionSpaceTransforms.Num();
	const int32 NumGeometries = RestCollection.NumElements(FGeometryCollection::GeometryGroup);

	TArray<TUniquePtr<FTriangleMesh>> TriangleMeshesArray;	//use to union trimeshes in cluster case
	TriangleMeshesArray.AddDefaulted(NumTransforms);

	TArray<FMassProperties> MassPropertiesArray;
	MassPropertiesArray.AddUninitialized(NumGeometries);

	// The geometry group has a set of transform indices that maps a geometry index
	// to a transform index, but only in the case where there is a 1-to-1 mapping 
	// between the two.  In the event where a geometry is instanced for multiple
	// transforms, the transform index on the geometry group should be INDEX_NONE.
	// Otherwise, iterating over the geometry group is a convenient way to iterate
	// over all the leaves of the hierarchy.
	check(!TransformIndex.Contains(INDEX_NONE)); // TODO: implement support for instanced bodies

	TArray<bool> IsTooSmallGeometryArray;
	IsTooSmallGeometryArray.Init(false, NumGeometries);

	GenerateInnerAndOuterRadiiIfNeeded(RestCollection);

	for (int32 TransformGroupIndex = 0; TransformGroupIndex < NumTransforms; TransformGroupIndex++)
	{
		if (SimulationType[TransformGroupIndex] == FGeometryCollection::ESimulationTypes::FST_None)
		{
			CollectionSimulatableParticles[TransformGroupIndex] = false;
		}
	}
	
	// compute mass properties in parallel
	ComputeMassPropertiesAndTriMeshes(RestCollection, SharedParams, TriangleMeshesArray, MassPropertiesArray, IsTooSmallGeometryArray);
	
	// compute total volume and whether we need to skip small geometry ( can't use parallelfors )
	bool bSkippedSmallGeometry = false;
	FReal TotalVolume = 0.f;
	for (int32 GeometryIndex = 0; GeometryIndex < NumGeometries; GeometryIndex++)
	{
		const int32 TransformGroupIndex = TransformIndex[GeometryIndex];
		if (CollectionSimulatableParticles[TransformGroupIndex])
		{
			TotalVolume += MassPropertiesArray[GeometryIndex].Volume;
		}
		// We skip very small geometry and log as a warning. To avoid log spamming, we wait
		// until we complete the loop before reporting the skips.
		if (IsTooSmallGeometryArray[GeometryIndex])
		{
			CollectionSimulatableParticles[TransformGroupIndex] = false;
			bSkippedSmallGeometry = true;
		}
	}

	// log any warning necessary
	if (bSkippedSmallGeometry)
	{
		UE_LOGF(LogChaos, Warning, "Some geometry is too small to be simulated and has been skipped.");
	}

	const TManagedArray<Chaos::FImplicitObjectPtr>* ExternaCollisions = RestCollection.FindAttribute<Chaos::FImplicitObjectPtr>(FGeometryCollection::ExternalCollisionsAttribute, FGeometryCollection::TransformGroup);
	
	// User provides us with total mass or density.
	// Density must be the same for individual parts and the total. Density_i = Density = Mass_i / Volume_i
	// Total mass must equal sum of individual parts. Mass_i = TotalMass * Volume_i / TotalVolume => Density_i = TotalMass / TotalVolume
	TotalVolume = FMath::Max(TotalVolume, SharedParams.MinimumVolumeClamp());
	const FReal DesiredTotalMass = SharedParams.bMassAsDensity ? SharedParams.Mass * TotalVolume : SharedParams.Mass;
	const FReal ClampedTotalMass = FMath::Clamp(DesiredTotalMass, SharedParams.MinimumMassClamp, SharedParams.MaximumMassClamp);
	const FReal DesiredDensity = ClampedTotalMass / TotalVolume;

	ParallelFor(NumGeometries, [&](int32 GeometryIndex)
	{
		// Need a new error reporter for parallel for loop here as it wouldn't be thread-safe to write to the prefix
		Chaos::FErrorReporter LocalErrorReporter;
		const int32 TransformGroupIndex = TransformIndex[GeometryIndex];

		const FReal Volume_i = MassPropertiesArray[GeometryIndex].Volume;
		if (CollectionSimulatableParticles[TransformGroupIndex])
		{
			// Must clamp each individual mass regardless of desired density
			if (DesiredDensity * Volume_i > SharedParams.MaximumMassClamp)
			{
				// For rigid bodies outside of range just defaut to a clamped bounding box, and warn the user.
				LocalErrorReporter.ReportError(*FString::Printf(TEXT("Geometry has invalid mass (too large)")));
				LocalErrorReporter.HandleLatestError();

				CollectionSimulatableParticles[TransformGroupIndex] = false;
			}
		}

		// compute the rest of the mass property ( Mass, InertiaTensor and RotationOfMass )
		if (CollectionSimulatableParticles[TransformGroupIndex])
		{
			TUniquePtr<FTriangleMesh>& TriMesh = TriangleMeshesArray[TransformGroupIndex];
			FMassProperties& MassProperties = MassPropertiesArray[GeometryIndex];

			const FReal Mass_i = FMath::Max(DesiredDensity * Volume_i, SharedParams.MinimumMassClamp);
			const FReal Density_i = Mass_i / Volume_i;
			CollectionMass[TransformGroupIndex] = (FRealSingle)Mass_i;

			// scale the mass properties inertia tensor by the Density_i ( as we computedit earlier with a density of one )
			MassPropertiesArray[GeometryIndex].InertiaTensor *= Density_i;
			const Chaos::FMatrix33& InertiaTensor = MassPropertiesArray[GeometryIndex].InertiaTensor;
			CollectionInertiaTensor[TransformGroupIndex] = FVector3f((float)InertiaTensor.M[0][0], (float)InertiaTensor.M[1][1], (float)InertiaTensor.M[2][2]);

			CollectionMassToLocal[TransformGroupIndex] = FTransform(MassProperties.RotationOfMass, MassProperties.CenterOfMass);
		}
	});

	// we need the vertices in mass space to compute the collision particles
	// todo(chaos) we shoudl eventually get rid of that that's a waste of memory and performance
	FParticles MassSpaceParticles;
	MassSpaceParticles.AddParticles(Vertex.Num());
	for (int32 Idx = 0; Idx < Vertex.Num(); ++Idx)
	{
		MassSpaceParticles.SetX(Idx, Vertex[Idx]);	//mass space computation done later down
	}
	
	
	// compute the bounding box in mesh space
	FVec3 MaxChildBounds(1);
	ParallelFor(NumGeometries, [&](int32 GeometryIndex)
	{
		const int32 TransformGroupIndex = TransformIndex[GeometryIndex];
		if (CollectionSimulatableParticles[TransformGroupIndex])
		{
			const TUniquePtr<Chaos::FTriangleMesh>& TriMesh = TriangleMeshesArray[GeometryIndex];
			const FTransform& MassToLocalTransform = CollectionMassToLocal[TransformGroupIndex];

			// Orient particle in the local mass space
			const bool bIdentityMassTransform = MassToLocalTransform.Equals(FTransform::Identity);
			if (!bIdentityMassTransform)
			{
				const int32 IdxStart = VertexStart[GeometryIndex];
				const int32 IdxEnd = IdxStart + VertexCount[GeometryIndex];
				for (int32 Idx = IdxStart; Idx < IdxEnd; ++Idx)
				{
					MassSpaceParticles.SetX(Idx, MassToLocalTransform.InverseTransformPositionNoScale(MassSpaceParticles.GetX(Idx)));
				}
			}

			FBox InstanceBoundingBox(EForceInit::ForceInitToZero);
			if (TriMesh->GetElements().Num())
			{
				// really not optimized as this generate a full set of the vertices with all the allocation that comes with it
				// the cost of going through all the vertices multiple time  may be faster 
				const TSet<int32> MeshVertices = TriMesh->GetVertices();
				for (const int32 Idx : MeshVertices)
				{
					InstanceBoundingBox += MassSpaceParticles.GetX(Idx);
				}
			}
			else if(VertexCount[GeometryIndex])
			{
				const int32 IdxStart = VertexStart[GeometryIndex];
				const int32 IdxEnd = IdxStart + VertexCount[GeometryIndex];
				for (int32 Idx = IdxStart; Idx < IdxEnd; ++Idx)
				{
					InstanceBoundingBox += MassSpaceParticles.GetX(Idx);
				}
			}
			else
			{
				const Chaos::FVec3 CenterOfMass = MassPropertiesArray[GeometryIndex].CenterOfMass;
				InstanceBoundingBox = FBox(CenterOfMass, CenterOfMass);
			}

			const FSharedSimulationSizeSpecificData& SizeSpecificData = GetSizeSpecificData(SharedParams.SizeSpecificData, RestCollection, TransformGroupIndex, InstanceBoundingBox);
			if (SizeSpecificData.CollisionShapesData.Num())
			{
				// Need a new error reporter for parallel for loop here as it wouldn't be thread-safe to write to the prefix
				Chaos::FErrorReporter LocalErrorReporter;
				//
				//  Build the simplicial for the rest collection. This will be used later in the DynamicCollection to 
				//  populate the collision structures of the simulation. 
				//
				if (ensureMsgf(TriMesh, TEXT("No Triangle representation")))
				{
					Chaos::FBVHParticles* Simplicial =
						FCollisionStructureManager::NewSimplicial(
							MassSpaceParticles,
							BoneMap,
							SizeSpecificData.CollisionShapesData[0].CollisionType,
							*TriMesh,
							SizeSpecificData.CollisionShapesData[0].CollisionParticleData.CollisionParticlesFraction);
					CollectionSimplicials[TransformGroupIndex] = TUniquePtr<FSimplicial>(Simplicial); // CollectionSimplicials is in the TransformGroup
					//ensureMsgf(CollectionSimplicials[TransformGroupIndex], TEXT("No simplicial representation."));
					if (!CollectionSimplicials[TransformGroupIndex]->Size())
					{
						ensureMsgf(false, TEXT("Simplicial is empty."));
					}

					// tdo(chaos) : if imported is selected but no shape exists shoudl we simply ignore ? 
					if (SharedParams.bUseImportedCollisionImplicits && ExternaCollisions && (*ExternaCollisions)[TransformGroupIndex])
					{
						// for now  simply copy the shared pointer as the imported geometry should not change under the hood without being recreated
						const Chaos::FImplicitObjectPtr ExternalCollisionImplicit = (*ExternaCollisions)[TransformGroupIndex];
						if (!bIdentityMassTransform)
						{
							// since we do not set the rotation of mass and center of mass properties on the particle and have a MasstoLocal managed property on the collection instead
							// we need to reverse transform the external shapes 
							CollectionImplicits[TransformGroupIndex] = MakeTransformImplicitObject(*ExternalCollisionImplicit, MassToLocalTransform.Inverse());
						}
						else
						{
							CollectionImplicits[TransformGroupIndex] = ExternalCollisionImplicit;
						}
					}
					else
					{
						LocalErrorReporter.SetPrefix(BaseErrorPrefix + " | Transform Index: " + FString::FromInt(TransformGroupIndex) + " of " + FString::FromInt(TransformIndex.Num()));
						CollectionImplicits[TransformGroupIndex] = CreateImplicitGeometry(
							SizeSpecificData,
							TransformGroupIndex,
							RestCollection,
							MassSpaceParticles,
							*TriMesh,
							InstanceBoundingBox,
							InnerRadius[GeometryIndex],
							LocalErrorReporter);
					}
				}
			}
		}
	});

	// Accumulate MaxChildBounds sequentially here rather than inside the ParallelFor above, where the
	// shared read-modify-write was a data race that made cooked output non-deterministic. Max is
	// associative and commutative, so this matches a race-free reduction.
	for (int32 GeometryIndex = 0; GeometryIndex < NumGeometries; ++GeometryIndex)
	{
		const int32 TransformGroupIndex = TransformIndex[GeometryIndex];
		if (CollectionSimulatableParticles[TransformGroupIndex])
		{
			if (const Chaos::FImplicitObjectPtr& Implicit = CollectionImplicits[TransformGroupIndex])
			{
				if (Implicit->HasBoundingBox())
				{
					MaxChildBounds = MaxChildBounds.ComponentwiseMax(Implicit->BoundingBox().Extents());
				}
			}
		}
	}

	// question: at the moment we always build cluster data in the asset. This 
	// allows for per instance toggling. Is this needed? It increases memory 
	// usage for all geometry collection assets.
	const bool bEnableClustering = true;	
	if (bEnableClustering)
	{
		//Put all children into collection space so we can compute mass properties.
		TUniquePtr<TPBDRigidClusteredParticles<FReal, 3>> CollectionSpaceParticles(new TPBDRigidClusteredParticles<FReal, 3>());
		CollectionSpaceParticles->AddParticles(NumTransforms);

		// Init to -FLT_MAX for debugging purposes
		for (int32 Idx = 0; Idx < NumTransforms; Idx++)
		{
			CollectionSpaceParticles->SetX(Idx, Chaos::FVec3(-TNumericLimits<FReal>::Max()));
		}

		//
		// TODO: We generate particles & handles for leaf nodes so that we can use some 
		// runtime clustering functions.  That's adding a lot of work and dependencies
		// just so we can make an API happy.  We should refactor the common routines
		// to have a handle agnostic implementation.
		//

		TMap<const TGeometryParticleHandle<FReal, 3>*, int32> HandleToTransformIdx;
		TArray<TUniquePtr<TPBDRigidClusteredParticleHandle<FReal, 3>>> Handles;
		Handles.Reserve(NumTransforms);
		for (int32 Idx = 0; Idx < NumTransforms; Idx++)
		{
			Handles.Add(TPBDRigidClusteredParticleHandle<FReal, 3>::CreateParticleHandle(
				MakeSerializable(CollectionSpaceParticles), Idx, Idx));
			HandleToTransformIdx.Add(Handles[Handles.Num() - 1].Get(), Idx);
		}

		// We use PopulateSimulatedParticle here just to give us some valid particles to operate on - with correct
		// position, mass and inertia so we can accumulate data for clusters just below.
 		for (int32 GeometryIdx = 0; GeometryIdx < NumGeometries; ++GeometryIdx)
 		{
 			const int32 TransformGroupIndex = TransformIndex[GeometryIdx];

 			if (CollectionSimulatableParticles[TransformGroupIndex])
 			{
				FTransform CollectionSpaceParticleTransform = CollectionMassToLocal[TransformGroupIndex] * CollectionSpaceTransforms[TransformGroupIndex];

 				PopulateSimulatedParticle(
 					Handles[TransformGroupIndex].Get(),
 					SharedParams, 
 					CollectionSimplicials[TransformGroupIndex].Get(),
 					CollectionImplicits[TransformGroupIndex],
 					Chaos::Filter::FShapeFilterData(),
 					Chaos::Filter::FInstanceData(),
 					CollectionMass[TransformGroupIndex],
 					CollectionInertiaTensor[TransformGroupIndex], 
					CollectionSpaceParticleTransform,
 					(uint8)EObjectStateTypeEnum::Chaos_Object_Dynamic, 
 					INDEX_NONE,  // CollisionGroup
					1.0f // todo(chaos) CollisionParticlesPerObjectFraction is not accessible right there for now but we can pass 1.0 for the time being
				);
 			}
 		}

		const TArray<int32> RecursiveOrder = GeometryCollectionAlgo::ComputeRecursiveOrder(RestCollection);
		const TArray<int32> TransformToGeometry = ComputeTransformToGeometryMap(RestCollection);

		TArray<bool> IsClusterSimulated;
		IsClusterSimulated.Init(false, CollectionSpaceParticles->Size());
		//build collision structures depth first
		for (const int32 TransformGroupIndex : RecursiveOrder)
		{
			if (RestCollection.IsClustered(TransformGroupIndex))
			{
				const int32 ClusterTransformIdx = TransformGroupIndex;
				//update mass 
				TSet<TPBDRigidParticleHandle<FReal,3>*> ChildrenIndices;
				{ // tmp scope
					ChildrenIndices.Reserve(Children[ClusterTransformIdx].Num());
					for (int32 ChildIdx : Children[ClusterTransformIdx])
					{
						if (CollectionSimulatableParticles[ChildIdx] || IsClusterSimulated[ChildIdx])
						{
							ChildrenIndices.Add(Handles[ChildIdx].Get());
						}
					}
					if (!ChildrenIndices.Num())
					{
						continue;
					}
				} // tmp scope

				//CollectionSimulatableParticles[TransformGroupIndex] = true;
				IsClusterSimulated[TransformGroupIndex] = true;

				// NOTE: Particles are in collection space for this function

				// Initialize XR to the collection space transform for this parent particle.
				// We have the individual mass properties for the children, but not for the parent
				// yet, so MassToLocal is not available.
				FPBDRigidClusteredParticleHandle* ClusterHandle = Handles[ClusterTransformIdx].Get();
				const FRigidTransform3 OriginalXR = CollectionSpaceTransforms[ClusterTransformIdx];
				ClusterHandle->SetX(OriginalXR.GetTranslation());
				ClusterHandle->SetR(OriginalXR.GetRotation());

				// This method will compute a mass and inertia, and center and rotation of mass and
				// store them on ClusterHandle based on the combined mass properties of its children.
				UpdateClusterMassProperties(ClusterHandle, ChildrenIndices);

				// NOTE: This method will is used to force an axis-aligned inertia tensor in local space,
				// (ie, zero rotation of inertia) which avoids a problem with generating bounds for GCs
				// when there's a mass rotation. The problem should go away once we remove MassToLocal
				// and replace it with regular old CenterOfMass and RotationOfMass.
				if (GeometryCollectionLocalInertiaDropOffDiagonalTerms)
				{
					AdjustClusterInertia(ClusterHandle, EInertiaOperations::LocalInertiaDropOffDiagonalTerms);
				}

				// Zero out CoM and RoM, and update XR accordingly so that when UpdateClusterMassProperties
				// is called in the next level up with ClusterHandle as one of the the children it's XR will
				// be correct.
				//
				// Store the relative transform from the particle's configuration space to its
				// MassToLocal because GeometryCollections assume that the particle's XR is at its
				// CoM and RoM - MassToLocal accounts for this offset.
				CollectionMassToLocal[ClusterTransformIdx] = MoveClusterToMassOffset(ClusterHandle, EMassOffsetType::Position | EMassOffsetType::Rotation);

				//update geometry
				//merge children meshes and move them into cluster's mass space
				TArray<TVector<int32, 3>> UnionMeshIndices;
				int32 BiggestNumElements = 0;
				{ // tmp scope
					int32 NumChildIndices = 0;
					for (TPBDRigidParticleHandle<FReal, 3>* Child : ChildrenIndices)
					{
						const int32 ChildTransformIdx = HandleToTransformIdx[Child];
						if (Chaos::FTriangleMesh* ChildMesh = TriangleMeshesArray[ChildTransformIdx].Get())
						{
							BiggestNumElements = FMath::Max(BiggestNumElements, ChildMesh->GetNumElements());
							NumChildIndices += ChildMesh->GetNumElements();
						}
					}
					UnionMeshIndices.Reserve(NumChildIndices);
				} // tmp scope

				FBox InstanceBoundingBox(EForceInit::ForceInitToZero);
				{ // tmp scope
					TArray<int32> VertsAdded;
					VertsAdded.Reserve(BiggestNumElements);
					for (TPBDRigidParticleHandle<FReal, 3>* Child : ChildrenIndices)
					{
						const int32 ChildTransformIdx = HandleToTransformIdx[Child];
						if (Chaos::FTriangleMesh* ChildMesh = TriangleMeshesArray[ChildTransformIdx].Get())
						{
							const TArray<TVector<int32, 3>>& ChildIndices = ChildMesh->GetSurfaceElements();
							UnionMeshIndices.Append(ChildIndices);

							// To move a particle from mass-space in the child to mass-space in the cluster parent, calculate
							// the relative transform between the mass-space origin for both the parent and child before
							// transforming the mass space particles into the parent mass-space.
							const FTransform ChildMassToClusterMass = (CollectionMassToLocal[ChildTransformIdx] * CollectionSpaceTransforms[ChildTransformIdx]).GetRelativeTransform(CollectionMassToLocal[ClusterTransformIdx] * CollectionSpaceTransforms[ClusterTransformIdx]);

							ChildMesh->GetVertexSetAsArray(VertsAdded);
							for (const int32 VertIdx : VertsAdded)
							{
								//Update particles so they are in the cluster's mass space
								MassSpaceParticles.SetX(VertIdx, 
									ChildMassToClusterMass.TransformPosition(MassSpaceParticles.GetX(VertIdx)));
								InstanceBoundingBox += MassSpaceParticles.GetX(VertIdx);
							}

							// Reset vert set for next child so we don't repeatedly transform verts and over extend our calculated bounds
							VertsAdded.Reset();
						}
					}
				} // tmp scope

				TUniquePtr<FTriangleMesh> UnionMesh(new FTriangleMesh(MoveTemp(UnionMeshIndices)));
				// TODO: Seems this should rotate full matrix and not discard off diagonals.
				const FVec3& InertiaDiagonal = CollectionSpaceParticles->I(ClusterTransformIdx);
				CollectionInertiaTensor[ClusterTransformIdx] = FVector3f(InertiaDiagonal);	// LWC_TODO: Precision loss
				CollectionMass[ClusterTransformIdx] = (FRealSingle)CollectionSpaceParticles->M(ClusterTransformIdx);

				const FSharedSimulationSizeSpecificData& SizeSpecificData = GetSizeSpecificData(SharedParams.SizeSpecificData, RestCollection, TransformGroupIndex, InstanceBoundingBox);

				int32 SizeSpecificIdx;
				if (bUseRelativeSize)
				{
					const TManagedArray<float>& RelativeSize = RestCollection.GetAttribute<float>(TEXT("Size"), FTransformCollection::TransformGroup);
					SizeSpecificIdx = GeometryCollection::SizeSpecific::FindIndexForVolume(SharedParams.SizeSpecificData, RelativeSize[TransformGroupIndex]);
				}
				else
				{
					SizeSpecificIdx = GeometryCollection::SizeSpecific::FindIndexForVolume(SharedParams.SizeSpecificData, InstanceBoundingBox);	
				}

				ErrorReporter.SetPrefix(BaseErrorPrefix + " | Cluster Transform Index: " + FString::FromInt(ClusterTransformIdx));

				if (SharedParams.bUseImportedCollisionImplicits && ExternaCollisions && (*ExternaCollisions)[TransformGroupIndex])
				{
					const FTransform& ClusterMassToLocal = CollectionMassToLocal[ClusterTransformIdx];
					const bool bIdentityMassTransform = ClusterMassToLocal.Equals(FTransform::Identity);

					// for now  simply copy the shared pointer as the imported geometry should not change under the hood without being recreated
					const Chaos::FImplicitObjectPtr ExternalCollisionImplicit = (*ExternaCollisions)[TransformGroupIndex];
					if (!bIdentityMassTransform)
					{
						// since we do not set the rotation of mass and center of mass properties on the particle and have a MasstoLocal managed property on the collection instead
						// we need to reverse transform the external shapes 
						CollectionImplicits[TransformGroupIndex] = MakeTransformImplicitObject(*ExternalCollisionImplicit, ClusterMassToLocal.Inverse());
					}
					else
					{
						CollectionImplicits[ClusterTransformIdx] = (*ExternaCollisions)[ClusterTransformIdx];
					}
				}
				else
				{
					CollectionImplicits[ClusterTransformIdx] = CreateImplicitGeometry(
						SizeSpecificData,
						ClusterTransformIdx,
						RestCollection,
						MassSpaceParticles,
						*UnionMesh,
						InstanceBoundingBox,
						InstanceBoundingBox.GetExtent().GetAbsMin(), // InnerRadius
						ErrorReporter,
						&MaxChildBounds
						);
				}
				// create simplicial from the implicit geometry
				CollectionSimplicials[ClusterTransformIdx] = TUniquePtr<FSimplicial>(
							FCollisionStructureManager::NewSimplicial(MassSpaceParticles, *UnionMesh, CollectionImplicits[ClusterTransformIdx].GetReference(),
							SharedParams.MaximumCollisionParticleCount));

				TriangleMeshesArray[ClusterTransformIdx] = MoveTemp(UnionMesh);
			}

			// Set level of TransformGroupIndex in Levels attribute.
			CalculateAndSetLevel(TransformGroupIndex, Parent, Levels);
		}
	}
}

void IdentifySimulatableElements(Chaos::FErrorReporter& ErrorReporter, FGeometryCollection& GeometryCollection)
{
	// Determine which collection particles to simulate

	// Geometry group
	const TManagedArray<int32>& TransformIndex = GeometryCollection.TransformIndex;
	const TManagedArray<FBox>& BoundingBox = GeometryCollection.BoundingBox;
	const TManagedArray<int32>& VertexCount = GeometryCollection.VertexCount;

	const int32 NumTransforms = GeometryCollection.NumElements(FGeometryCollection::TransformGroup);
	const int32 NumTransformMappings = TransformIndex.Num();

	// Faces group
	const TManagedArray<FIntVector>& Indices = GeometryCollection.Indices;
	const TManagedArray<bool>& Visible = GeometryCollection.Visible;
	// Vertices group
	const TManagedArray<int32>& BoneMap = GeometryCollection.BoneMap;

	// Do not simulate hidden geometry
	TArray<bool> HiddenObject;
	HiddenObject.Init(true, NumTransforms);
	int32 PrevObject = INDEX_NONE;
	bool bContiguous = true;
	for(int32 i = 0; i < Indices.Num(); i++)
	{
		if(Visible[i]) // Face index i is visible
		{
			const int32 ObjIdx = BoneMap[Indices[i][0]]; // Look up associated bone to the faces X coord.
			HiddenObject[ObjIdx] = false;

			if (!ensure(ObjIdx >= PrevObject))
			{
				bContiguous = false;
			}

			PrevObject = ObjIdx;
		}
	}

	if (!bContiguous)
	{
		// What assumptions???  How are we ever going to know if this is still the case?
		ErrorReporter.ReportError(TEXT("Objects are not contiguous. This breaks assumptions later in the pipeline"));
		ErrorReporter.HandleLatestError();
	}

	//For now all simulation data is a non compiled attribute. Not clear what we want for simulated vs kinematic collections
	TManagedArray<bool>& SimulatableParticles = 
		GeometryCollection.AddAttribute<bool>(
			FGeometryCollection::SimulatableParticlesAttribute, FTransformCollection::TransformGroup);
	SimulatableParticles.Fill(false);

	for(int i = 0; i < NumTransformMappings; i++)
	{
		int32 Tdx = TransformIndex[i];
		checkSlow(0 <= Tdx && Tdx < NumTransforms);
		if (GeometryCollection.IsGeometry(Tdx) && // checks that TransformToGeometryIndex[Tdx] != INDEX_NONE
			VertexCount[i] &&					 // must have vertices to be simulated?
			0.f < BoundingBox[i].GetSize().SizeSquared() && // must have a non-zero bbox to be simulated?  No single point?
			!HiddenObject[Tdx])					 // must have 1 associated face
		{
			SimulatableParticles[Tdx] = true;
		}
	}
}

void BuildSimulationData(Chaos::FErrorReporter& ErrorReporter, FGeometryCollection& GeometryCollection, const FSharedSimulationParameters& SharedParams)
{
	IdentifySimulatableElements(ErrorReporter, GeometryCollection);
	FGeometryCollectionPhysicsProxy::InitializeSharedCollisionStructures(ErrorReporter, GeometryCollection, SharedParams);
}

//==============================================================================
// FIELDS
//==============================================================================

FGeometryCollectionPhysicsProxy::FFieldData& FGeometryCollectionPhysicsProxy::GetOrCreateFieldData_Internal()
{
	if (!FieldData_Internal)
	{
		FieldData_Internal = MakeUnique<FFieldData>();
	}
	return *FieldData_Internal;
}

void FGeometryCollectionPhysicsProxy::BufferFieldCommand_External(FFieldSystemCommand&& Command)
{
	check(IsInGameThread());
	if (Chaos::FPhysicsSolver* RBDSolver = GetSolver<Chaos::FPhysicsSolver>())
	{
		RBDSolver->EnqueueCommandImmediate(
			[this, RBDSolver, FieldCommand = MoveTemp(Command)]()
			{
				BufferFieldCommand_Internal(RBDSolver, FieldCommand);
			});
	}
}

// deprecated - simple forward to BufferFieldCommand_Internal
void FGeometryCollectionPhysicsProxy::BufferCommand(Chaos::FPBDRigidsSolver* RigidsSolver, const FFieldSystemCommand& Command)
{
	BufferFieldCommand_Internal(RigidsSolver, Command);
}

void FGeometryCollectionPhysicsProxy::BufferFieldCommand_Internal(Chaos::FPBDRigidsSolver* RigidsSolver, const FFieldSystemCommand& Command)
{
	check(RigidsSolver != nullptr);
	RigidsSolver->GetGeometryCollectionPhysicsProxiesField_Internal().Add(this);
	GetOrCreateFieldData_Internal().Commands.Add(Command);
}

void FGeometryCollectionPhysicsProxy::FieldParameterUpdateCallback(Chaos::FPBDRigidsSolver* RigidSolver, const bool bUpdateViews)
{
	SCOPE_CYCLE_COUNTER(STAT_ParamUpdateField_Object);
	check(RigidSolver);
	// We are updating the Collection from the InitializeBodiesPT, so we need the PT collection
	FGeometryDynamicCollection& Collection = PhysicsThreadCollection;
	Chaos::FPBDPositionConstraints PositionTarget;
	TMap<int32, int32> TargetedParticles;

	FFieldData& FieldData = GetOrCreateFieldData_Internal();

	// Process Particle-Collection commands
	int32 NumCommands = FieldData.Commands.Num();
	if (NumCommands && !RigidSolver->IsShuttingDown() && Collection.GetNumTransforms())
	{
		TArray<int32> CommandsToRemove;
		CommandsToRemove.Reserve(NumCommands);

		EFieldResolutionType PrevResolutionType = EFieldResolutionType::Field_Resolution_Max;
		EFieldFilterType PrevFilterType = EFieldFilterType::Field_Filter_Max;
		EFieldObjectType PrevObjectType = EFieldObjectType::Field_Object_Max;
		EFieldPositionType PrevPositionType = EFieldPositionType::Field_Position_Max;

		for (int32 CommandIndex = 0; CommandIndex < NumCommands; CommandIndex++)
		{
			FFieldSystemCommand& FieldCommand = FieldData.Commands[CommandIndex];
			if (IsParameterFieldValid(FieldCommand) || FieldCommand.PhysicsType == EFieldPhysicsType::Field_InitialLinearVelocity || FieldCommand.PhysicsType == EFieldPhysicsType::Field_InitialAngularVelocity)
			{
				if (Chaos::BuildFieldSamplePoints(this, RigidSolver, FieldCommand, FieldData.ExecutionDatas, PrevResolutionType, PrevFilterType, PrevObjectType, PrevPositionType))
				{
					const Chaos::FReal TimeSeconds = RigidSolver->GetSolverTime() - FieldCommand.TimeCreation;

					FFieldContext FieldContext(
						FieldData.ExecutionDatas,
						FieldCommand.MetaData,
						TimeSeconds);

					TArray<Chaos::FGeometryParticleHandle*>& ParticleHandles = FieldData.ExecutionDatas.ParticleHandles[(uint8)EFieldCommandHandlesType::InsideHandles];

					if (FieldCommand.RootNode->Type() == FFieldNodeBase::EFieldType::EField_Int32)
					{
						TArray<int32>& FinalResults = FieldData.ExecutionDatas.IntegerResults[(uint8)EFieldCommandResultType::FinalResult];
						ResetResultsArray < int32 >(FieldData.ExecutionDatas.SamplePositions.Num(), FinalResults, 0);

						TFieldArrayView<int32> ResultsView(FinalResults, 0, FinalResults.Num());

						if (FieldCommand.PhysicsType == EFieldPhysicsType::Field_DynamicState)
						{
							SCOPE_CYCLE_COUNTER(STAT_ParamUpdateField_DynamicState);
							{
								InitDynamicStateResults(ParticleHandles, FieldContext, FinalResults);

								static_cast<const FFieldNode<int32>*>(FieldCommand.RootNode.Get())->Evaluate(FieldContext, ResultsView);

								bool bHasStateChanged = false;
								
								const FGeometryDynamicCollection::FInitialVelocityFacade InitialVelocityFacade = Collection.GetInitialVelocityFacade();

								const TFieldArrayView<FFieldContextIndex>& EvaluatedSamples = FieldContext.GetEvaluatedSamples();
								for (const FFieldContextIndex& Index : EvaluatedSamples)
								{
									Chaos::TPBDRigidParticleHandle<Chaos::FReal, 3>* RigidHandle = ParticleHandles[Index.Sample]->CastToRigidParticle();
									if (RigidHandle)
									{
										const int32 CurrResult = ResultsView[Index.Result];
										check(CurrResult <= std::numeric_limits<int8>::max() &&
											CurrResult >= std::numeric_limits<int8>::min());

										const int8 ResultState = static_cast<int8>(CurrResult);
										const int32 TransformIndex = GetTransformGroupIndexFromHandle(RigidHandle);
										if (TransformIndex != INDEX_NONE)
										{
											Chaos::FVec3 InitialLinearVelocity{ 0 };
											Chaos::FVec3 InitialAngularVelocity{ 0 };

											// bUpdateViews is used here to avoid applying the velocity twice when running the Initial fields ( see InitializeBodiesPT )
											const bool bHasInitialVelocities = (bUpdateViews && (Parameters.InitialVelocityType == EInitialVelocityTypeEnum::Chaos_Initial_Velocity_User_Defined));
											if (bHasInitialVelocities)
											{
												InitialLinearVelocity = InitialVelocityFacade.InitialLinearVelocityAttribute[TransformIndex];
												InitialAngularVelocity = InitialVelocityFacade.InitialAngularVelocityAttribute[TransformIndex];
											}

											// Update of the handles object state. 
											bHasStateChanged |= ReportDynamicStateResult(RigidSolver, static_cast<Chaos::EObjectStateType>(ResultState), RigidHandle,
												bHasInitialVelocities, InitialLinearVelocity, bHasInitialVelocities, InitialAngularVelocity);

											// Update of the Collection dynamic state. It will be used just after to set the initial velocity
											Collection.DynamicState[TransformIndex] = ResultState;
										}
									}
								}
								if (bUpdateViews && bHasStateChanged)
								{
									UpdateSolverParticlesState(RigidSolver, EvaluatedSamples, ParticleHandles);
								}
							}
						}
						else
						{
							Chaos::FieldIntegerParameterUpdate(RigidSolver, FieldCommand, FieldData.ExecutionDatas.ParticleHandles[(uint8)EFieldCommandHandlesType::InsideHandles],
								FieldContext, PositionTarget, TargetedParticles, FinalResults);
						}
					}
					else if (FieldCommand.RootNode->Type() == FFieldNodeBase::EFieldType::EField_FVector)
					{
						TArray<FVector>& FinalResults = FieldData.ExecutionDatas.VectorResults[(uint8)EFieldCommandResultType::FinalResult];
						ResetResultsArray < FVector >(FieldData.ExecutionDatas.SamplePositions.Num(), FinalResults, FVector::ZeroVector);

						TFieldArrayView<FVector> ResultsView(FinalResults, 0, FinalResults.Num());

						if (FieldCommand.PhysicsType == EFieldPhysicsType::Field_InitialLinearVelocity)
						{
							if (Parameters.InitialVelocityType == EInitialVelocityTypeEnum::Chaos_Initial_Velocity_User_Defined)
							{
								SCOPE_CYCLE_COUNTER(STAT_ParamUpdateField_LinearVelocity);
								{
									FGeometryDynamicCollection::FInitialVelocityFacade InitialVelocityFacade = Collection.GetInitialVelocityFacade();
									if (InitialVelocityFacade.IsValid())
									{
										static_cast<const FFieldNode<FVector>*>(FieldCommand.RootNode.Get())->Evaluate(FieldContext, ResultsView);
										for (const FFieldContextIndex& Index : FieldContext.GetEvaluatedSamples())
										{
											Chaos::TPBDRigidParticleHandle<Chaos::FReal, 3>* RigidHandle = ParticleHandles[Index.Sample]->CastToRigidParticle();
											if (RigidHandle)
											{
												const int32 TransformIndex = GetTransformGroupIndexFromHandle(RigidHandle);
												if (TransformIndex != INDEX_NONE)
												{
													InitialVelocityFacade.InitialLinearVelocityAttribute.ModifyAt(TransformIndex, FVector3f(ResultsView[Index.Result]));
												}
											}
										}
									}
									else
									{
										UE_LOGF(LogChaos, Warning, "Geometry collection : trying to apply a InitialLinearVelocity field but InitialVelocityType is not set to None")
									}
								}
							}
							else
							{
								UE_LOGF(LogChaos, Error, "Field based evaluation of the simulations 'InitialLinearVelocity' requires the geometry collection be set to User Defined Initial Velocity");
							}
						}
						else if (FieldCommand.PhysicsType == EFieldPhysicsType::Field_InitialAngularVelocity)
						{
							if (Parameters.InitialVelocityType == EInitialVelocityTypeEnum::Chaos_Initial_Velocity_User_Defined)
							{
								SCOPE_CYCLE_COUNTER(STAT_ParamUpdateField_AngularVelocity);
								{
									FGeometryDynamicCollection::FInitialVelocityFacade InitialVelocityFacade = Collection.GetInitialVelocityFacade();
									if (InitialVelocityFacade.IsValid())
									{
										static_cast<const FFieldNode<FVector>*>(FieldCommand.RootNode.Get())->Evaluate(FieldContext, ResultsView);
										for (const FFieldContextIndex& Index : FieldContext.GetEvaluatedSamples())
										{
											Chaos::TPBDRigidParticleHandle<Chaos::FReal, 3>* RigidHandle = ParticleHandles[Index.Sample]->CastToRigidParticle();
											if (RigidHandle)
											{
												const int32 TransformIndex = GetTransformGroupIndexFromHandle(RigidHandle);
												if (TransformIndex != INDEX_NONE)
												{
													InitialVelocityFacade.InitialAngularVelocityAttribute.ModifyAt(TransformIndex, FVector3f(ResultsView[Index.Result]));
												}
											}
										}
									}
									else
									{
										UE_LOGF(LogChaos, Warning, "Geometry collection : trying to apply a InitialLinearVelocity field but InitialVelocityType is not set to None")
									}
								}
							}
							else
							{
								UE_LOGF(LogChaos, Error, "Field based evaluation of the simulations 'InitialAngularVelocity' requires the geometry collection be set to User Defined Initial Velocity");
							}
						}
						else
						{
							Chaos::FieldVectorParameterUpdate(RigidSolver, FieldCommand, ParticleHandles,
								FieldContext, PositionTarget, TargetedParticles, FinalResults);
						}
					}
					else if (FieldCommand.RootNode->Type() == FFieldNodeBase::EFieldType::EField_Float)
					{
						TArray<float>& FinalResults = FieldData.ExecutionDatas.ScalarResults[(uint8)EFieldCommandResultType::FinalResult];
						ResetResultsArray<float>(FieldData.ExecutionDatas.SamplePositions.Num(), FinalResults, 0.0f);

						TFieldArrayView<float> ResultsView(FinalResults, 0, FinalResults.Num());

						Chaos::FieldScalarParameterUpdate(RigidSolver, FieldCommand, ParticleHandles,
							FieldContext, PositionTarget, TargetedParticles, FinalResults);
					}
				}
				CommandsToRemove.Add(CommandIndex);
			}
		}		
		
		for (int32 Index = CommandsToRemove.Num() - 1; Index >= 0; --Index)
		{
			FieldData.Commands.RemoveAt(CommandsToRemove[Index]);
		}
	}
}

void FGeometryCollectionPhysicsProxy::FieldForcesUpdateCallback(Chaos::FPBDRigidsSolver* RigidSolver)
{
	SCOPE_CYCLE_COUNTER(STAT_ForceUpdateField_Object);
	check(RigidSolver);

	FFieldData& FieldData = GetOrCreateFieldData_Internal();

	const int32 NumCommands = FieldData.Commands.Num();
	if (NumCommands && !RigidSolver->IsShuttingDown())
	{
		TArray<int32> CommandsToRemove;
		CommandsToRemove.Reserve(NumCommands);

		EFieldResolutionType PrevResolutionType = EFieldResolutionType::Field_Resolution_Max;
		EFieldFilterType PrevFilterType = EFieldFilterType::Field_Filter_Max;
		EFieldObjectType PrevObjectType = EFieldObjectType::Field_Object_Max;
		EFieldPositionType PrevPositionType = EFieldPositionType::Field_Position_Max;

		for (int32 CommandIndex = 0; CommandIndex < NumCommands; CommandIndex++)
		{
			const FFieldSystemCommand& FieldCommand = FieldData.Commands[CommandIndex];
			if (IsForceFieldValid(FieldCommand))
			{
				if (Chaos::BuildFieldSamplePoints(this, RigidSolver, FieldCommand, FieldData.ExecutionDatas, PrevResolutionType, PrevFilterType, PrevObjectType, PrevPositionType))
				{
					const Chaos::FReal TimeSeconds = RigidSolver->GetSolverTime() - FieldCommand.TimeCreation;

					FFieldContext FieldContext(
						FieldData.ExecutionDatas,
						FieldCommand.MetaData,
						TimeSeconds);

					TArray<Chaos::FGeometryParticleHandle*>& ParticleHandles = FieldData.ExecutionDatas.ParticleHandles[(uint8)EFieldCommandHandlesType::InsideHandles];

					if (FieldCommand.RootNode->Type() == FFieldNode<FVector>::StaticType())
					{
						TArray<FVector>& FinalResults = FieldData.ExecutionDatas.VectorResults[(uint8)EFieldCommandResultType::FinalResult];
						ResetResultsArray < FVector >(FieldData.ExecutionDatas.SamplePositions.Num(), FinalResults, FVector::ZeroVector);

						Chaos::FieldVectorForceUpdate(RigidSolver, FieldCommand, ParticleHandles,
							FieldContext, FinalResults);
					}
				}
				CommandsToRemove.Add(CommandIndex);
			}
		}
		for (int32 Index = CommandsToRemove.Num() - 1; Index >= 0; --Index)
		{
			FieldData.Commands.RemoveAt(CommandsToRemove[Index]);
		}
	}
}

FGeometryCollectionPhysicsProxy::FClusterHandle* FGeometryCollectionPhysicsProxy::GetInitialRootHandle_Internal() const
{
	const int32 Index = FromTransformToParticleIndex[Parameters.InitialRootIndex];
	if (SolverParticleHandles.IsValidIndex(Index))
	{
		return SolverParticleHandles[Index];
	}
	return nullptr;
}

TArray<Chaos::FPhysicsObjectHandle> FGeometryCollectionPhysicsProxy::GetAllPhysicsObjects() const
{
	TArray<Chaos::FPhysicsObjectHandle> Handles;
	Handles.Reserve(NumEffectiveParticles);

	for (const Chaos::FPhysicsObjectUniquePtr& Object : PhysicsObjects)
	{
		if (Object.Get() != nullptr)
		{
			Handles.Add(Object.Get());
		}
	}
	return Handles;
}

TArray<Chaos::FPhysicsObjectHandle> FGeometryCollectionPhysicsProxy::GetAllPhysicsObjectIncludingNulls() const
{
	TArray<Chaos::FPhysicsObjectHandle> Handles;
	Handles.SetNum(NumTransforms);

	for (int32 TransformIndex = 0; TransformIndex < NumTransforms; ++TransformIndex)
	{
		const int32 ParticleIndex = FromTransformToParticleIndex[TransformIndex];
		if (ParticleIndex == INDEX_NONE)
		{
			Handles[TransformIndex] = nullptr;
		}
		else
		{
			Handles[TransformIndex] = PhysicsObjects[ParticleIndex].Get();
		}
	}
	return Handles;
}

Chaos::FPhysicsObjectHandle FGeometryCollectionPhysicsProxy::GetPhysicsObjectByIndex(int32 Index) const
{
	if (FromTransformToParticleIndex.IsValidIndex(Index))
	{
		const int32 ParticleIndex = FromTransformToParticleIndex[Index];
		if (ParticleIndex != INDEX_NONE)
		{
			return PhysicsObjects[ParticleIndex].Get();
		}
	}
	return nullptr; 
}

FGeometryCollectionPhysicsProxy::FParticle* FGeometryCollectionPhysicsProxy::GetParticleByIndex_External(int32 Index)
{
	return const_cast<FGeometryCollectionPhysicsProxy::FParticle*>(const_cast<const FGeometryCollectionPhysicsProxy*>(this)->GetParticleByIndex_External(Index));
}

const FGeometryCollectionPhysicsProxy::FParticle* FGeometryCollectionPhysicsProxy::GetParticleByIndex_External(int32 Index) const
{
	if (FromTransformToParticleIndex.IsValidIndex(Index))
	{
		const int32 ParticleIndex = FromTransformToParticleIndex[Index];
		if (ParticleIndex != INDEX_NONE)
		{
			return GTParticles[ParticleIndex].Get();
		}
	}
	return nullptr;
}

FGeometryCollectionPhysicsProxy::FParticleHandle* FGeometryCollectionPhysicsProxy::GetParticleByIndex_Internal(int32 Index)
{
	return const_cast<FGeometryCollectionPhysicsProxy::FParticleHandle*>(const_cast<const FGeometryCollectionPhysicsProxy*>(this)->GetParticleByIndex_Internal(Index));
}

const FGeometryCollectionPhysicsProxy::FParticleHandle* FGeometryCollectionPhysicsProxy::GetParticleByIndex_Internal(int32 Index) const
{
	if (FromTransformToParticleIndex.IsValidIndex(Index))
	{
		const int32 ParticleIndex = FromTransformToParticleIndex[Index];
		if (ParticleIndex != INDEX_NONE)
		{
			return SolverParticleHandles[ParticleIndex];
		}
	}
	return nullptr;
}


void FGeometryCollectionPhysicsProxy::RebaseAllGameThreadCollectionTransformsOnNewWorldTransform_External()
{
	check(IsInGameThread());
	const TManagedArray<bool>* AnimationsActive = GameThreadCollection.GetAnimateTransformAttribute();
	const TManagedArray<FTransform>& MassToLocal = Parameters.RestCollectionShared->GetAttribute<FTransform>(MassToLocalAttributeName, FTransformCollection::TransformGroup);
	const bool bIsComponentTransformScaled = !WorldTransform_External.GetScale3D().Equals(FVector::OneVector);
	const FTransform ComponentScaleTransform(FQuat::Identity, FVector::ZeroVector, WorldTransform_External.GetScale3D());

	for (int32 ParticleIndex = 0; ParticleIndex < NumEffectiveParticles; ++ParticleIndex)
	{
		const int32 TransformIndex = FromParticleToTransformIndex[ParticleIndex];
		const bool bAnimatingWhileDisabled = AnimationsActive ? (*AnimationsActive)[TransformIndex] : false;
		const bool bActive = GameThreadCollection.Active[TransformIndex];
		if (bActive || bAnimatingWhileDisabled)
		{
			if (GTParticles[ParticleIndex] != nullptr)
			{
				RebaseParticleGameThreadCollectionTransformOnNewWorldTransform_External(ParticleIndex, MassToLocal, bIsComponentTransformScaled, ComponentScaleTransform);
			}
		}
	}
}

bool FGeometryCollectionPhysicsProxy::RebaseParticleGameThreadCollectionTransformOnNewWorldTransform_External(int32 ParticleIndex, const TManagedArray<FTransform>& MassToLocal, bool bIsComponentTransformScaled, const FTransform& ComponentScaleTransform)
{
	check(IsInGameThread());
	bool bHasDifferentTransforms = false;
	const int32 TransformIndex = FromParticleToTransformIndex[ParticleIndex];
	const FParticle& GTParticle = *GTParticles[ParticleIndex];
	const FTransform& ParticleMassToLocal = MassToLocal[TransformIndex];
	const int32 ParentTransformIndex = GameThreadCollection.GetParent(TransformIndex);

	const FTransform& WorldTransform = ParticleMassToLocal.Inverse() * FTransform { GTParticle.R(), GTParticle.X() };

	// by default parent is the component's world transform 
	FTransform ParentWorldTransform = WorldTransform_External;
	if (ParentTransformIndex != INDEX_NONE && FromTransformToParticleIndex[ParentTransformIndex] != INDEX_NONE)
	{
		if (const FParticle* GTParentParticle = GTParticles[FromTransformToParticleIndex[ParentTransformIndex]].Get())
		{
			const FTransform& ParentMassToLocal = MassToLocal[TransformIndex];
			ParentWorldTransform = ParentMassToLocal.Inverse() * FTransform { GTParentParticle->R(), GTParentParticle->X() };
		}
	}

	FTransform NewTransform = WorldTransform.GetRelativeTransform(ParentWorldTransform);
	if (ParentTransformIndex == INDEX_NONE && bIsComponentTransformScaled)
	{
		NewTransform = ParticleMassToLocal.Inverse() * ComponentScaleTransform * ParticleMassToLocal * NewTransform;
	}
	if (!NewTransform.Equals(FTransform(GameThreadCollection.GetTransform(TransformIndex)), GeometryCollectionTransformTolerance))
	{
		bHasDifferentTransforms = true;
		GameThreadCollection.SetTransform(TransformIndex, FTransform3f(NewTransform));
	}
	return bHasDifferentTransforms;
}

void FDamageCollector::Reset(int32 NumTransforms)
{
	DamageData.Reset();
	DamageData.SetNum(NumTransforms);
}

void FDamageCollector::SampleDamage(int32 TransformIndex, float Damage, float DamageThreshold)
{
	if (DamageData.IsValidIndex(TransformIndex))
	{
		FDamageData& Data = DamageData[TransformIndex];
		Data.DamageThreshold = DamageThreshold;
		Data.MaxDamages = FMath::Max(Data.MaxDamages, Damage);
		Data.bIsBroken = Data.bIsBroken || (Damage > DamageThreshold);
	}
}

FRuntimeDataCollector& FRuntimeDataCollector::GetInstance()
{
	static FRuntimeDataCollector Instance;
	return Instance;
}

void FRuntimeDataCollector::Clear()
{
	Collectors.Reset();
}

FDamageCollector* FRuntimeDataCollector::Find(const FGuid& Guid)
{
	return Collectors.Find(Guid);
}

void FRuntimeDataCollector::AddCollector(const FGuid& Guid, int32 TransformNum)
{
	FDamageCollector& Collector = Collectors.FindOrAdd(Guid);
	Collector.Reset(TransformNum);
}

void FRuntimeDataCollector::RemoveCollector(const FGuid& Guid)
{
	Collectors.Remove(Guid);
	
}
