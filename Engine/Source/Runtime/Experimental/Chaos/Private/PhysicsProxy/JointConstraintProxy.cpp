// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsProxy/JointConstraintProxy.h"

#include "ChaosStats.h"
#include "Chaos/Collision/SpatialAccelerationBroadPhase.h"
#include "Chaos/Collision/CollisionConstraintFlags.h"
#include "Chaos/ErrorReporter.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/GeometryParticles.h"
#include "Chaos/Serializable.h"
#include "Chaos/PBDJointConstraints.h"
#include "Chaos/Framework/MultiBufferResource.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "PhysicsSolver.h"
#include "Chaos/PullPhysicsDataImp.h"
#include "Chaos/PhysicsObjectInternal.h"

namespace Chaos
{

FJointConstraintPhysicsProxy::FJointConstraintPhysicsProxy(FJointConstraint* InConstraint, FPBDJointConstraintHandle* InHandle, UObject* InOwner)
	: Base(EPhysicsProxyType::JointConstraintType, InOwner, MakeShared<FProxyTimestampBase>())
	, Constraint_GT(InConstraint)	// This proxy assumes ownership of the Constraint, and will free it in DestroyOnGameThread
	, Constraint_PT(InHandle)		// This proxy assumes ownership of the Handle, and will free it in DestroyOnPhysicsThread
	, OriginalParticleHandles_PT(nullptr, nullptr)
{
	check(Constraint_GT !=nullptr);
	Constraint_GT->SetProxy(this);
}

FGeometryParticleHandle*
FJointConstraintPhysicsProxy::GetParticleHandleFromProxy(IPhysicsProxyBase* ProxyBase)
{
	if (ProxyBase)
	{
		if (ProxyBase->GetType() == EPhysicsProxyType::SingleParticleProxy)
		{
			return ((FSingleParticlePhysicsProxy*)ProxyBase)->GetHandle_LowLevel();
		}
	}
	return nullptr;
}

/**/
void FJointConstraintPhysicsProxy::BufferPhysicsResults(FDirtyJointConstraintData& Buffer)
{
	Buffer.SetProxy(*this);
	if (Constraint_PT != nullptr && (Constraint_PT->IsValid() || Constraint_PT->IsConstraintBreaking() || Constraint_PT->IsDriveTargetChanged()))
	{
		Buffer.OutputData.bIsBreaking = Constraint_PT->IsConstraintBreaking();
		Buffer.OutputData.bIsBroken = !Constraint_PT->IsConstraintEnabled();
		Buffer.OutputData.bIsViolating = Constraint_PT->IsConstraintViolating();
		Buffer.OutputData.bDriveTargetChanged = Constraint_PT->IsDriveTargetChanged();
		Buffer.OutputData.Force = Constraint_PT->GetLinearImpulse();
		Buffer.OutputData.Torque = Constraint_PT->GetAngularImpulse();
		Buffer.OutputData.LinearViolation = Constraint_PT->GetLinearViolation();
		Buffer.OutputData.AngularViolation = Constraint_PT->GetAngularViolation();

		Constraint_PT->ClearConstraintBreaking(); // it's a single frame event, so reset
		Constraint_PT->ClearConstraintViolating();
		Constraint_PT->ClearDriveTargetChanged(); // it's a single frame event, so reset
	}
}

/**/
bool FJointConstraintPhysicsProxy::PullFromPhysicsState(const FDirtyJointConstraintData& Buffer, const int32 SolverSyncTimestamp)
{
	if (Constraint_GT != nullptr && Constraint_GT->IsValid())
	{
		if (Buffer.OutputData.bIsBreaking || Buffer.OutputData.bDriveTargetChanged)
		{
			Constraint_GT->GetOutputData().bIsBreaking = Buffer.OutputData.bIsBreaking;
			Constraint_GT->GetOutputData().bIsBroken = Buffer.OutputData.bIsBroken;
			Constraint_GT->GetOutputData().bDriveTargetChanged = Buffer.OutputData.bDriveTargetChanged;
		}
		Constraint_GT->GetOutputData().Force = Buffer.OutputData.Force;
		Constraint_GT->GetOutputData().Torque = Buffer.OutputData.Torque;

		Constraint_GT->GetOutputData().bIsViolating = Buffer.OutputData.bIsViolating;
		Constraint_GT->GetOutputData().LinearViolation = Buffer.OutputData.LinearViolation;
		Constraint_GT->GetOutputData().AngularViolation = Buffer.OutputData.AngularViolation;
	}

	return true;
}

template <typename TransformType>
static void FixConnectorTransformsForRoot(const FParticlePair& OriginalHandles, const FParticlePair& RootHandles, TransformType& InOutTransformsToFix)
{
	for (int32 Index = 0; Index < 2; Index++)
	{
		if (OriginalHandles[Index] && RootHandles[Index] && RootHandles[Index] != OriginalHandles[Index])
		{
			const FTransform TransformOffset = OriginalHandles[Index]->GetTransformXR().GetRelativeTransform(RootHandles[Index]->GetTransformXR());
			InOutTransformsToFix[Index] *= TransformOffset;
		}
	}
}

void FJointConstraintPhysicsProxy::InitializeOnPhysicsThread(FPBDRigidsSolver* InSolver, FDirtyPropertiesManager& Manager, int32 DataIdx, FDirtyChaosProperties& RemoteData)
{
	auto& Handles = InSolver->GetParticles().GetParticleHandles();
	if (Handles.Size())
	{
		auto& JointConstraints = InSolver->GetJointCombinedConstraints();
		if(const FPhysicsObjectPairProperty* BodyPairs = RemoteData.FindJointPhysicsObjects(Manager, DataIdx))
		{
			FParticlePair RootHandles
			{
				BodyPairs->PhysicsBodies[0]->GetRootParticle<Chaos::EThreadContext::Internal>(),
				BodyPairs->PhysicsBodies[1]->GetRootParticle<Chaos::EThreadContext::Internal>(),
			};

			if (RootHandles[0] && RootHandles[1])
			{
				if (const FPBDJointSettings* SrcJointSettings = RemoteData.FindJointSettings(Manager, DataIdx))
				{
					FPBDJointSettings JointSettings = *SrcJointSettings;

					// if the root particles do not match the actual particles
					// we need to adjust the frames to be in the root particle space 
					OriginalParticleHandles_PT =
					{
						BodyPairs->PhysicsBodies[0]->GetParticle<Chaos::EThreadContext::Internal>(),
						BodyPairs->PhysicsBodies[1]->GetParticle<Chaos::EThreadContext::Internal>(),
					};
					FixConnectorTransformsForRoot(OriginalParticleHandles_PT, RootHandles, JointSettings.ConnectorTransforms);

					Constraint_PT = InSolver->GetEvolution()->CreateJointConstraint(RootHandles, JointSettings);
				}
			}
		}
	}
}

void FJointConstraintPhysicsProxy::DestroyOnPhysicsThread(FPBDRigidsSolver* InSolver)
{
	if (Constraint_PT)
	{
		InSolver->GetEvolution()->DestroyJointConstraint(Constraint_PT);

		Constraint_PT = nullptr;
		OriginalParticleHandles_PT = { nullptr, nullptr };
	}
}

void FJointConstraintPhysicsProxy::DestroyOnGameThread()
{
	delete Constraint_GT;
	Constraint_GT = nullptr;
}


void FJointConstraintPhysicsProxy::PushStateOnGameThread(FDirtyPropertiesManager& Manager, int32 DataIdx, FDirtyChaosProperties& RemoteData)
{
	if (Constraint_GT && Constraint_GT->IsValid())
	{
		Constraint_GT->SyncRemoteData(Manager, DataIdx, RemoteData);
	}
}


void FJointConstraintPhysicsProxy::PushStateOnPhysicsThread(FPBDRigidsSolver* InSolver, const FDirtyPropertiesManager& Manager, int32 DataIdx, const FDirtyChaosProperties& RemoteData)
{
	if (Constraint_PT && Constraint_PT->IsValid())
	{
		if (const FPBDJointSettings* Data = RemoteData.FindJointSettings(Manager, DataIdx))
		{
			FPBDJointSettings JointSettingsToSet = *Data;

			// PushStateOnPhysicsThread is always called right after InitializeOnPhysicsThread
			// so we need to make sure the root space correction logic is also run here to avoid resetting the transform in the wrong space
			FixConnectorTransformsForRoot(OriginalParticleHandles_PT, Constraint_PT->GetConstrainedParticles(), JointSettingsToSet.ConnectorTransforms);

			// NOTE: If bUseLinearSolver changes, the handle get re-allocated
			Constraint_PT = InSolver->GetEvolution()->SetJointConstraintSettings(Constraint_PT, JointSettingsToSet);
		}
	}
}

}