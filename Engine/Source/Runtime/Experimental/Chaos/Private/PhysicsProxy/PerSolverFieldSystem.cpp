// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsProxy/PerSolverFieldSystem.h"
#include "PhysicsProxy/GeometryCollectionPhysicsProxy.h"
#include "Field/FieldSystem.h"
#include "Chaos/PBDRigidClustering.h"
#include "GeometryCollection/GeometryCollectionSimulationTypes.h"
#include "PhysicsProxy/FieldSystemProxyHelper.h"
#include "PhysicsSolver.h"
#include "ChaosStats.h"

void ResetIndicesArray(TArray<int32>& IndicesArray, int32 Size)
{
	if (IndicesArray.Num() != Size)
	{
		IndicesArray.SetNum(Size);
		for (int32 i = 0; i < IndicesArray.Num(); ++i)
		{
			IndicesArray[i] = i;
		}
	}
}  

//==============================================================================
// FPerSolverFieldSystem
//==============================================================================

void FPerSolverFieldSystem::FieldParameterUpdateInternal(
	Chaos::FPBDRigidsSolver* RigidSolver,
	Chaos::FPBDPositionConstraints& PositionTarget,
	TMap<int32, int32>& TargetedParticles,
	TArray<FFieldSystemCommand>& Commands, const bool IsTransient)
{
	SCOPE_CYCLE_COUNTER(STAT_ParamUpdateField_Object);

	const int32 NumCommands = Commands.Num();
	if (NumCommands && RigidSolver)
	{
		TArray<int32> CommandsToRemove;
		CommandsToRemove.Reserve(NumCommands);

		EFieldResolutionType PrevResolutionType = EFieldResolutionType::Field_Resolution_Max;
		EFieldFilterType PrevFilterType = EFieldFilterType::Field_Filter_Max;
		EFieldObjectType PrevObjectType = EFieldObjectType::Field_Object_Max;
		EFieldPositionType PrevPositionType = EFieldPositionType::Field_Position_Max;

		for (int32 CommandIndex = 0; CommandIndex < NumCommands; CommandIndex++)
		{
			const FFieldSystemCommand& FieldCommand = Commands[CommandIndex];
			if(IsParameterFieldValid(FieldCommand))
			{
				if (Chaos::BuildFieldSamplePoints(this, RigidSolver, FieldCommand, ExecutionDatas, PrevResolutionType, PrevFilterType, PrevObjectType, PrevPositionType))
				{
					const Chaos::FReal TimeSeconds = RigidSolver->GetSolverTime() - FieldCommand.TimeCreation;

					FFieldContext FieldContext(
						ExecutionDatas,
						FieldCommand.MetaData,
						TimeSeconds);

					TArray<Chaos::FGeometryParticleHandle*>& ParticleHandles = ExecutionDatas.ParticleHandles[(uint8)EFieldCommandHandlesType::InsideHandles];

					if (FieldCommand.RootNode->Type() == FFieldNodeBase::EFieldType::EField_Int32)
					{
						TArray<int32>& FinalResults = ExecutionDatas.IntegerResults[(uint8)EFieldCommandResultType::FinalResult];
						ResetResultsArray < int32 >(ExecutionDatas.SamplePositions.Num(), FinalResults, 0);

						Chaos::FieldIntegerParameterUpdate(RigidSolver, FieldCommand, ParticleHandles,
							FieldContext, PositionTarget, TargetedParticles, FinalResults);
					}
					else if (FieldCommand.RootNode->Type() == FFieldNodeBase::EFieldType::EField_Float)
					{
						TArray<float>& FinalResults = ExecutionDatas.ScalarResults[(uint8)EFieldCommandResultType::FinalResult];
						ResetResultsArray < float >(ExecutionDatas.SamplePositions.Num(), FinalResults, 0.0);

						Chaos::FieldScalarParameterUpdate(RigidSolver, FieldCommand, ParticleHandles,
							FieldContext, PositionTarget, TargetedParticles, FinalResults);
					}
					else if (FieldCommand.RootNode->Type() == FFieldNodeBase::EFieldType::EField_FVector)
					{
						TArray<FVector>& FinalResults = ExecutionDatas.VectorResults[(uint8)EFieldCommandResultType::FinalResult];
						ResetResultsArray < FVector >(ExecutionDatas.SamplePositions.Num(), FinalResults, FVector::ZeroVector);

						Chaos::FieldVectorParameterUpdate(RigidSolver, FieldCommand, ParticleHandles,
							FieldContext, PositionTarget, TargetedParticles, FinalResults);
					}
				}
				CommandsToRemove.Add(CommandIndex);
			}
		}
		if (IsTransient)
		{
			for (int32 Index = CommandsToRemove.Num() - 1; Index >= 0; --Index)
			{
				Commands.RemoveAt(CommandsToRemove[Index]);
			}
		}
	}
}



void FPerSolverFieldSystem::FieldParameterUpdateCallback(
	Chaos::FPBDRigidsSolver* InSolver,
	Chaos::FPBDPositionConstraints& PositionTarget,
	TMap<int32, int32>& TargetedParticles)
{
	if (InSolver && !InSolver->IsShuttingDown())
	{
		FieldParameterUpdateInternal(InSolver, PositionTarget, TargetedParticles, TransientCommands, true);
		FieldParameterUpdateInternal(InSolver, PositionTarget, TargetedParticles, PersistentCommands, false);
	}
}

void FPerSolverFieldSystem::FieldForcesUpdateInternal(
	Chaos::FPBDRigidsSolver* RigidSolver,
	TArray<FFieldSystemCommand>& Commands, const bool IsTransient)
{
	SCOPE_CYCLE_COUNTER(STAT_ForceUpdateField_Object);

	const int32 NumCommands = Commands.Num();
	if (NumCommands && RigidSolver)
	{
		TArray<int32> CommandsToRemove;
		CommandsToRemove.Reserve(NumCommands);

		EFieldResolutionType PrevResolutionType = EFieldResolutionType::Field_Resolution_Max;
		EFieldFilterType PrevFilterType = EFieldFilterType::Field_Filter_Max;
		EFieldObjectType PrevObjectType = EFieldObjectType::Field_Object_Max;
		EFieldPositionType PrevPositionType = EFieldPositionType::Field_Position_Max;

		for (int32 CommandIndex = 0; CommandIndex < NumCommands; CommandIndex++)
		{
			const FFieldSystemCommand& FieldCommand = Commands[CommandIndex];

			if (IsForceFieldValid(FieldCommand))
			{
				if (Chaos::BuildFieldSamplePoints(this, RigidSolver, FieldCommand, ExecutionDatas, PrevResolutionType, PrevFilterType, PrevObjectType, PrevPositionType))
				{
					const Chaos::FReal TimeSeconds = RigidSolver->GetSolverTime() - FieldCommand.TimeCreation;

					FFieldContext FieldContext(
						ExecutionDatas,
						FieldCommand.MetaData,
						TimeSeconds);

					TArray<Chaos::FGeometryParticleHandle*>& ParticleHandles = ExecutionDatas.ParticleHandles[(uint8)EFieldCommandHandlesType::InsideHandles];

					if (FieldCommand.RootNode->Type() == FFieldNodeBase::EFieldType::EField_FVector)
					{
						TArray<FVector>& FinalResults = ExecutionDatas.VectorResults[(uint8)EFieldCommandResultType::FinalResult];
						ResetResultsArray < FVector >(ExecutionDatas.SamplePositions.Num(), FinalResults, FVector::ZeroVector);

						Chaos::FieldVectorForceUpdate(RigidSolver, FieldCommand, ParticleHandles,
							FieldContext, FinalResults);
					}
				}
				CommandsToRemove.Add(CommandIndex);
			}
		}
		if (IsTransient)
		{
			for (int32 Index = CommandsToRemove.Num() - 1; Index >= 0; --Index)
			{
				Commands.RemoveAt(CommandsToRemove[Index]);
			}
		}
	}
}

void FPerSolverFieldSystem::FieldForcesUpdateCallback(
	Chaos::FPBDRigidsSolver* InSolver)
{
	if (InSolver && !InSolver->IsShuttingDown())
	{
		FieldForcesUpdateInternal(InSolver, TransientCommands, true);
		FieldForcesUpdateInternal(InSolver, PersistentCommands, false);
	}
}

FORCEINLINE void EvaluateImpulseField(
	const FFieldSystemCommand& FieldCommand,
	FFieldContext& FieldContext,
	TFieldArrayView<FVector>& ResultsView,
	TArray<FVector>& OutputImpulse)
{
	static_cast<const FFieldNode<FVector>*>(FieldCommand.RootNode.Get())->Evaluate(FieldContext, ResultsView);
	if (OutputImpulse.Num() == 0)
	{
		OutputImpulse.SetNumZeroed(ResultsView.Num(), EAllowShrinking::No);
		for (const FFieldContextIndex& Index : FieldContext.GetEvaluatedSamples())
		{
			if (Index.Sample < OutputImpulse.Num() && Index.Result < ResultsView.Num())
			{
				OutputImpulse[Index.Sample] = ResultsView[Index.Result];
			}
		}
	}
	else
	{
		for (const FFieldContextIndex& Index : FieldContext.GetEvaluatedSamples())
		{
			if (Index.Sample < OutputImpulse.Num() && Index.Result < ResultsView.Num())
			{
				OutputImpulse[Index.Sample] += ResultsView[Index.Result];
			}
		}
	}
}

void ComputeFieldRigidImpulseInternal(
	FFieldExecutionDatas& ExecutionDatas,
	const Chaos::FReal SolverTime,
	TArray<FFieldSystemCommand>& Commands, const bool IsTransient)
{
	const int32 NumCommands = Commands.Num();
	if (NumCommands)
	{
		TArray<int32> CommandsToRemove;
		CommandsToRemove.Reserve(NumCommands);

		for (int32 CommandIndex = 0; CommandIndex < NumCommands; CommandIndex++)
		{
			const FFieldSystemCommand& FieldCommand = Commands[CommandIndex];

			EFieldObjectType ObjectType = EFieldObjectType::Field_Object_Max;
			if (FieldCommand.HasMetaData(FFieldSystemMetaData::EMetaType::ECommandData_Filter))
			{
				ObjectType = FieldCommand.GetMetaDataAs<FFieldSystemMetaDataFilter>(FFieldSystemMetaData::EMetaType::ECommandData_Filter)->ObjectType;
			}

			if ((ObjectType == EFieldObjectType::Field_Object_Character) || (ObjectType == EFieldObjectType::Field_Object_All) || (ObjectType == EFieldObjectType::Field_Object_Max))
			{
				const Chaos::FReal TimeSeconds = SolverTime - FieldCommand.TimeCreation;

				FFieldContext FieldContext(
					ExecutionDatas,
					FieldCommand.MetaData,
					TimeSeconds);

				if (FieldCommand.RootNode->Type() == FFieldNodeBase::EFieldType::EField_FVector)
				{
					TArray<FVector>& FinalResults = ExecutionDatas.VectorResults[(uint8)EFieldCommandResultType::FinalResult];
					ResetResultsArray < FVector >(ExecutionDatas.SamplePositions.Num(), FinalResults, FVector::ZeroVector);

					TFieldArrayView<FVector> ResultsView(FinalResults, 0, FinalResults.Num());

					if (FieldCommand.PhysicsType == EFieldPhysicsType::Field_LinearVelocity)
					{
						SCOPE_CYCLE_COUNTER(STAT_ParamUpdateField_LinearVelocity);

						EvaluateImpulseField(FieldCommand, FieldContext, ResultsView, ExecutionDatas.FieldOutputs[(uint8)EFieldCommandOutputType::LinearVelocity]);
					}
					else if (FieldCommand.PhysicsType == EFieldPhysicsType::Field_LinearForce)
					{
						SCOPE_CYCLE_COUNTER(STAT_ForceUpdateField_LinearForce);

						EvaluateImpulseField(FieldCommand, FieldContext, ResultsView, ExecutionDatas.FieldOutputs[(uint8)EFieldCommandOutputType::LinearForce]);
					}
					if (FieldCommand.PhysicsType == EFieldPhysicsType::Field_AngularVelociy)
					{
						SCOPE_CYCLE_COUNTER(STAT_ParamUpdateField_AngularVelocity);

						EvaluateImpulseField(FieldCommand, FieldContext, ResultsView, ExecutionDatas.FieldOutputs[(uint8)EFieldCommandOutputType::AngularVelocity]);
					}
					else if (FieldCommand.PhysicsType == EFieldPhysicsType::Field_AngularTorque)
					{
						SCOPE_CYCLE_COUNTER(STAT_ForceUpdateField_AngularTorque);

						EvaluateImpulseField(FieldCommand, FieldContext, ResultsView, ExecutionDatas.FieldOutputs[(uint8)EFieldCommandOutputType::AngularTorque]);
					}
				}
			}
			CommandsToRemove.Add(CommandIndex);
		}
		if (IsTransient)
		{
			for (int32 Index = CommandsToRemove.Num() - 1; Index >= 0; --Index)
			{
				Commands.RemoveAt(CommandsToRemove[Index]);
			}
		}
	}
}

void FPerSolverFieldSystem::ComputeFieldRigidImpulse(
	const Chaos::FReal SolverTime)
{
	static const TArray<EFieldCommandOutputType> EmptyTargets = { EFieldCommandOutputType::LinearVelocity,
																  EFieldCommandOutputType::LinearForce,
																  EFieldCommandOutputType::AngularVelocity,
																  EFieldCommandOutputType::AngularTorque };
	

	EmptyResultsArrays <FVector> (EmptyTargets, ExecutionDatas.FieldOutputs);

	ComputeFieldRigidImpulseInternal(ExecutionDatas, SolverTime, TransientCommands, true);
	ComputeFieldRigidImpulseInternal(ExecutionDatas, SolverTime, PersistentCommands, false);
}

void ComputeFieldLinearImpulseInternal(
	FFieldExecutionDatas& ExecutionDatas,
	const Chaos::FReal SolverTime,
	TArray<FFieldSystemCommand>& Commands, const bool IsTransient)
{
	const int32 NumCommands = Commands.Num();
	if (NumCommands)
	{
		TArray<int32> CommandsToRemove;
		CommandsToRemove.Reserve(NumCommands);

		for (int32 CommandIndex = 0; CommandIndex < NumCommands; CommandIndex++)
		{
			const FFieldSystemCommand& FieldCommand = Commands[CommandIndex];

			EFieldObjectType ObjectType = EFieldObjectType::Field_Object_Max;
			if (FieldCommand.HasMetaData(FFieldSystemMetaData::EMetaType::ECommandData_Filter))
			{
				ObjectType = FieldCommand.GetMetaDataAs<FFieldSystemMetaDataFilter>(FFieldSystemMetaData::EMetaType::ECommandData_Filter)->ObjectType;
			}

			if ((ObjectType == EFieldObjectType::Field_Object_Cloth) || (ObjectType == EFieldObjectType::Field_Object_All) || (ObjectType == EFieldObjectType::Field_Object_Max))
			{
				const Chaos::FReal TimeSeconds = SolverTime - FieldCommand.TimeCreation;

				FFieldContext FieldContext(
					ExecutionDatas,
					FieldCommand.MetaData,
					TimeSeconds);

				if (FieldCommand.RootNode->Type() == FFieldNodeBase::EFieldType::EField_FVector)
				{
					TArray<FVector>& FinalResults = ExecutionDatas.VectorResults[(uint8)EFieldCommandResultType::FinalResult];
					ResetResultsArray < FVector >(ExecutionDatas.SamplePositions.Num(), FinalResults, FVector::ZeroVector);

					TFieldArrayView<FVector> ResultsView(FinalResults, 0, FinalResults.Num());

					if (FieldCommand.PhysicsType == EFieldPhysicsType::Field_LinearVelocity)
					{
						SCOPE_CYCLE_COUNTER(STAT_ParamUpdateField_LinearVelocity);

						EvaluateImpulseField(FieldCommand, FieldContext, ResultsView, ExecutionDatas.FieldOutputs[(uint8)EFieldCommandOutputType::LinearVelocity]);
					}
					else if (FieldCommand.PhysicsType == EFieldPhysicsType::Field_LinearForce)
					{
						SCOPE_CYCLE_COUNTER(STAT_ForceUpdateField_LinearForce);

						EvaluateImpulseField(FieldCommand, FieldContext, ResultsView, ExecutionDatas.FieldOutputs[(uint8)EFieldCommandOutputType::LinearForce]);
					}
				}
			}
			CommandsToRemove.Add(CommandIndex);
		}
		if (IsTransient)
		{
			for (int32 Index = CommandsToRemove.Num() - 1; Index >= 0; --Index)
			{
				Commands.RemoveAt(CommandsToRemove[Index]);
			}
		}
	}
}

void FPerSolverFieldSystem::ComputeFieldLinearImpulse(const Chaos::FReal SolverTime)
{
	static const TArray<EFieldCommandOutputType> EmptyTargets = { EFieldCommandOutputType::LinearVelocity,
												EFieldCommandOutputType::LinearForce };

	EmptyResultsArrays<FVector>(EmptyTargets, ExecutionDatas.FieldOutputs);

	ComputeFieldLinearImpulseInternal(ExecutionDatas, SolverTime, TransientCommands, true);
	ComputeFieldLinearImpulseInternal(ExecutionDatas, SolverTime, PersistentCommands, false);
}

void FPerSolverFieldSystem::AddTransientCommand(const FFieldSystemCommand& FieldCommand)
{
	TransientCommands.Add(FieldCommand);
}

void FPerSolverFieldSystem::AddPersistentCommand(const FFieldSystemCommand& FieldCommand)
{
	PersistentCommands.Add(FieldCommand);
}

void FPerSolverFieldSystem::RemoveTransientCommand(const FFieldSystemCommand& FieldCommand)
{
	TransientCommands.Remove(FieldCommand);
}

void FPerSolverFieldSystem::RemovePersistentCommand(const FFieldSystemCommand& FieldCommand)
{
	PersistentCommands.Remove(FieldCommand);
}


void AddClusterChildren(TArray<Chaos::FGeometryParticleHandle*>& Handles,
						const Chaos::FPBDRigidsSolver* RigidSolver,
						const Chaos::TPBDRigidClusteredParticleHandleImp<Chaos::FReal, 3, false>* Clustered)
{
	using FClusterMap = Chaos::FRigidClustering::FClusterMap;
	using FParticleHandel = Chaos::TPBDRigidParticleHandle<Chaos::FReal, 3>;

	if (Clustered && Clustered->ClusterIds().NumChildren)
	{
		const FClusterMap& ClusterMap = RigidSolver->GetEvolution()->GetRigidClustering().GetChildrenMap();
		if (ClusterMap.Contains(Clustered->Handle()))
		{
			for (FParticleHandel* Child : ClusterMap[Clustered->Handle()])
			{
				Handles.Add(Child);
			}
		}
	}
}

void FPerSolverFieldSystem::GetRelevantParticleHandles(
	TArray<Chaos::FGeometryParticleHandle*>& Handles,
	const Chaos::FPBDRigidsSolver* RigidSolver,
	const EFieldResolutionType ResolutionType)
{
	Handles.SetNum(0, EAllowShrinking::No);
	const Chaos::FPBDRigidsSOAs& SolverParticles = RigidSolver->GetParticles();

	if (ResolutionType == EFieldResolutionType::Field_Resolution_Minimal)
	{
		const Chaos::TParticleView<Chaos::FGeometryParticles>& ParticleView =
			SolverParticles.GetNonDisabledView();
		Handles.Reserve(ParticleView.Num()); // ?? what about additional number of children added
		for (Chaos::TParticleIterator<Chaos::FGeometryParticles> It = ParticleView.Begin(), ItEnd = ParticleView.End();
			It != ItEnd; ++It)
		{
			const Chaos::TTransientGeometryParticleHandle<Chaos::FReal,3>* Handle = &(*It);
			Handles.Add(GetHandleHelper(const_cast<Chaos::TTransientGeometryParticleHandle<Chaos::FReal,3>*>(Handle)));
			AddClusterChildren(Handles, RigidSolver, It->CastToClustered());
		}
	}
	else if (ResolutionType == EFieldResolutionType::Field_Resolution_DisabledParents)
	{
		const auto& Clustering = RigidSolver->GetEvolution()->GetRigidClustering();
		const auto& ClusterMap = Clustering.GetChildrenMap();
		Handles.Reserve(Clustering.GetTopLevelClusterParents().Num());

		for (Chaos::TPBDRigidClusteredParticleHandle<Chaos::FReal, 3> * TopLevelParent : Clustering.GetTopLevelClusterParents())
		{
			Handles.Add(TopLevelParent);
		}
	}
	else if (ResolutionType == EFieldResolutionType::Field_Resolution_Maximum)
	{
		const Chaos::TParticleView<Chaos::FGeometryParticles>& ParticleView =
			SolverParticles.GetAllParticlesView();
		Handles.Reserve(ParticleView.Num());

		for (Chaos::TParticleIterator<Chaos::FGeometryParticles> It = ParticleView.Begin(), ItEnd = ParticleView.End();
			It != ItEnd; ++It)
		{
			const Chaos::TTransientGeometryParticleHandle<Chaos::FReal,3>* Handle = &(*It);
			Handles.Add(GetHandleHelper(const_cast<Chaos::TTransientGeometryParticleHandle<Chaos::FReal,3>*>(Handle)));
		}
	}
}

template<typename ParticleHandleType>
bool ValidateParticle(const EFieldObjectType ObjectType, const ParticleHandleType& ParticleHandle)
{
	using namespace Chaos;
	const EParticleType ParticleType = ParticleHandle->GetParticleType();
	const bool bIsDestructionParticle = (ParticleType == EParticleType::GeometryCollection || ParticleType == EParticleType::Clustered);
	return (ObjectType == EFieldObjectType::Field_Object_All)
		|| (ObjectType == EFieldObjectType::Field_Object_Max)
		|| ((ObjectType == EFieldObjectType::Field_Object_Rigid) && !bIsDestructionParticle)
		|| ((ObjectType == EFieldObjectType::Field_Object_Destruction) && bIsDestructionParticle);
}

void FPerSolverFieldSystem::GetFilteredParticleHandles(
	TArray<Chaos::FGeometryParticleHandle*>& Handles,
	const Chaos::FPBDRigidsSolver* RigidSolver,
	const EFieldFilterType FilterType,
	const EFieldObjectType ObjectType)
{
	Handles.SetNum(0, EAllowShrinking::No);
	const Chaos::FPBDRigidsSOAs& SolverParticles = RigidSolver->GetParticles();
	if (FilterType == EFieldFilterType::Field_Filter_Dynamic)
	{
		const Chaos::TParticleView<Chaos::TPBDRigidParticles<Chaos::FReal, 3>>& ParticleView =
			SolverParticles.GetNonDisabledDynamicView();
		Handles.Reserve(ParticleView.Num());

		for (Chaos::TParticleIterator<Chaos::TPBDRigidParticles<Chaos::FReal, 3>> It = ParticleView.Begin(), ItEnd = ParticleView.End();
			It != ItEnd; ++It)
		{
			if (!It->Sleeping() && ValidateParticle(ObjectType, It))
			{
				const Chaos::TTransientGeometryParticleHandle<Chaos::FReal, 3>* Handle = &(*It);
				Handles.Add(GetHandleHelper(const_cast<Chaos::TTransientGeometryParticleHandle<Chaos::FReal, 3>*>(Handle)));
				AddClusterChildren(Handles, RigidSolver, It->CastToClustered());
			}
		}
	}
	else if (FilterType == EFieldFilterType::Field_Filter_Static)
	{
		const Chaos::TParticleView<Chaos::FGeometryParticles>& ParticleView =
			SolverParticles.GetActiveStaticParticlesView();
		Handles.Reserve(ParticleView.Num());

		for (Chaos::TParticleIterator<Chaos::FGeometryParticles> It = ParticleView.Begin(), ItEnd = ParticleView.End();
			It != ItEnd; ++It)
		{
			if (ValidateParticle(ObjectType, It))
			{
				const Chaos::TTransientGeometryParticleHandle<Chaos::FReal, 3>* Handle = &(*It);
				Handles.Add(GetHandleHelper(const_cast<Chaos::TTransientGeometryParticleHandle<Chaos::FReal, 3>*>(Handle)));
			}
		}
	}
	else if (FilterType == EFieldFilterType::Field_Filter_Kinematic)
	{
		const Chaos::TParticleView<Chaos::FKinematicGeometryParticles>& ParticleView =
			SolverParticles.GetActiveKinematicParticlesView();
		Handles.Reserve(ParticleView.Num());

		for (Chaos::TParticleIterator<Chaos::FKinematicGeometryParticles> It = ParticleView.Begin(), ItEnd = ParticleView.End();
			It != ItEnd; ++It)
		{
			if (ValidateParticle(ObjectType, It))
			{
				const Chaos::TTransientGeometryParticleHandle<Chaos::FReal, 3>* Handle = &(*It);
				Handles.Add(GetHandleHelper(const_cast<Chaos::TTransientGeometryParticleHandle<Chaos::FReal, 3>*>(Handle)));
			}
		}
	}
	else if (FilterType == EFieldFilterType::Field_Filter_Sleeping)
	{
		const Chaos::TParticleView<Chaos::TPBDRigidParticles<Chaos::FReal, 3>>& ParticleView =
			SolverParticles.GetNonDisabledDynamicView();
		Handles.Reserve(ParticleView.Num());

		for (Chaos::TParticleIterator<Chaos::TPBDRigidParticles<Chaos::FReal, 3>> It = ParticleView.Begin(), ItEnd = ParticleView.End();
			It != ItEnd; ++It)
		{
			if (It->Sleeping() && ValidateParticle(ObjectType, It))
			{
				const Chaos::TTransientGeometryParticleHandle<Chaos::FReal, 3>* Handle = &(*It);
				Handles.Add(GetHandleHelper(const_cast<Chaos::TTransientGeometryParticleHandle<Chaos::FReal, 3>*>(Handle)));
			}
		}
	}
	else if (FilterType == EFieldFilterType::Field_Filter_Disabled)
	{
		const Chaos::TParticleView<Chaos::FGeometryParticles>& ParticleView =
			SolverParticles.GetAllParticlesView();
		Handles.Reserve(ParticleView.Num());

		for (Chaos::TParticleIterator<Chaos::FGeometryParticles> It = ParticleView.Begin(), ItEnd = ParticleView.End();
			It != ItEnd; ++It)
		{
			Chaos::TPBDRigidParticleHandleImp<Chaos::FReal, 3, false>* RigidHandle = It->CastToRigidParticle();
			if (RigidHandle && RigidHandle->Disabled() && ValidateParticle(ObjectType, RigidHandle))
			{
				const Chaos::TTransientGeometryParticleHandle<Chaos::FReal, 3>* Handle = &(*It);
				Handles.Add(GetHandleHelper(const_cast<Chaos::TTransientGeometryParticleHandle<Chaos::FReal, 3>*>(Handle)));
			}
		}
	}
	else if (FilterType == EFieldFilterType::Field_Filter_DynamicAndSleeping)
	{
		const Chaos::TParticleView<Chaos::TPBDRigidParticles<Chaos::FReal, 3>>& ParticleView =
			SolverParticles.GetNonDisabledDynamicView();
		Handles.Reserve(ParticleView.Num());

		for (Chaos::TParticleIterator<Chaos::TPBDRigidParticles<Chaos::FReal, 3>> It = ParticleView.Begin(), ItEnd = ParticleView.End();
			It != ItEnd; ++It)
		{
			if (ValidateParticle(ObjectType, It))
			{
				const Chaos::TTransientGeometryParticleHandle<Chaos::FReal, 3>* Handle = &(*It);
				Handles.Add(GetHandleHelper(const_cast<Chaos::TTransientGeometryParticleHandle<Chaos::FReal, 3>*>(Handle)));
				AddClusterChildren(Handles, RigidSolver, It->CastToClustered());
			}
		}
	}
	else if (FilterType == EFieldFilterType::Field_Filter_All)
	{
		const Chaos::TParticleView<Chaos::FGeometryParticles>& ParticleView =
			SolverParticles.GetNonDisabledView();
		Handles.Reserve(ParticleView.Num());

		for (Chaos::TParticleIterator<Chaos::FGeometryParticles> It = ParticleView.Begin(), ItEnd = ParticleView.End();
			It != ItEnd; ++It)
		{
			if (ValidateParticle(ObjectType, It))
			{
				const Chaos::TTransientGeometryParticleHandle<Chaos::FReal, 3>* Handle = &(*It);
				Handles.Add(GetHandleHelper(const_cast<Chaos::TTransientGeometryParticleHandle<Chaos::FReal, 3>*>(Handle)));
				AddClusterChildren(Handles, RigidSolver, It->CastToClustered());
			}
		}
	}
}
