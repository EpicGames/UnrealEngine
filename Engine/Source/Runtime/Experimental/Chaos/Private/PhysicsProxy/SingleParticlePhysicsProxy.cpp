// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsProxy/SingleParticlePhysicsProxy.h"

#include "ChaosStats.h"
#include "Chaos/Box.h"
#include "Chaos/Capsule.h"
#include "Chaos/Sphere.h"
#include "Chaos/ErrorReporter.h"
#include "Chaos/Serializable.h"
#include "Chaos/PerParticleGravity.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/PhysicsObjectInternal.h"
#include "Chaos/Framework/MultiBufferResource.h"
#include "PhysicsSolver.h"
#include "Chaos/ChaosMarshallingManager.h"
#include "Chaos/PullPhysicsDataImp.h"
#include "RewindData.h"
#include "Chaos/DebugDrawQueue.h"

namespace Chaos
{

// This allows forcing the game thread/actor to get or not get transform updates from the simulation result for
// kinematics - noting that they may already have been set in the SetKinematicTransform function.
// Velocity and such will still be copied in any case. The default value of -1 uses the UpdateKinematicFromSimulation 
// flag on the BodyInstance.
CHAOS_API int32 SyncKinematicOnGameThread = -1;
FAutoConsoleVariableRef CVar_SyncKinematicOnGameThread(TEXT("P.Chaos.SyncKinematicOnGameThread"), 
	SyncKinematicOnGameThread, TEXT(
		"If set to 1, kinematic bodies will always send their transforms back to the game thread, following the "
		"simulation step/results. If 0, then they will never do so, and kinematics will be updated immediately "
		"their kinematic target is set. Any other value (e.g. the default -1) means that the behavior is "
		"determined on a per-object basis with the UpdateKinematicFromSimulation flag in BodyInstance."));

static bool bPullPhysicsStateForceUpdateBounds = false;
static FAutoConsoleVariableRef CVar_PullPhysicsStateForceUpdateBounds(
	TEXT("P.Chaos.PullPhysicsStateForceUpdateBounds"),
	bPullPhysicsStateForceUpdateBounds,
	TEXT("If true, updates rigid particle bounds regardless of if X or R changed. The optimal setting is false.")
);

FSingleParticlePhysicsProxy::FSingleParticlePhysicsProxy(TUniquePtr<PARTICLE_TYPE>&& InParticle, FParticleHandle* InHandle, UObject* InOwner)
	: IPhysicsProxyBase(EPhysicsProxyType::SingleParticleProxy, InOwner, MakeShared<FSingleParticleProxyTimestamp>())
	, Particle(MoveTemp(InParticle))
	, Handle(InHandle)
{
	Particle->SetProxy(this);
	Reference = FPhysicsObjectFactory::CreatePhysicsObject(this);
	InterpolationData = MakeUnique<FProxyInterpolationBase>();
}


FSingleParticlePhysicsProxy::~FSingleParticlePhysicsProxy()
{
}

CHAOS_API int32 ForceNoCollisionIntoSQ = 0;
FAutoConsoleVariableRef CVarForceNoCollisionIntoSQ(TEXT("p.ForceNoCollisionIntoSQ"), ForceNoCollisionIntoSQ, TEXT("When enabled, all particles end up in sq structure, even ones with no collision"));

template <Chaos::EParticleType ParticleType>
void PushToPhysicsStateImp(const Chaos::FDirtyPropertiesManager& Manager, Chaos::FGeometryParticleHandle* Handle, int32 DataIdx, const Chaos::FDirtyProxy& Dirty, Chaos::FShapeDirtyData* ShapesData, Chaos::FPBDRigidsSolver& Solver, bool bResimInitialized, Chaos::FReal ExternalDt)
{
	using namespace Chaos;
	constexpr bool bHasKinematicData = ParticleType != EParticleType::Static;
	constexpr bool bHasDynamicData = ParticleType == EParticleType::Rigid;
	auto KinematicHandle = bHasKinematicData ? static_cast<Chaos::FKinematicGeometryParticleHandle*>(Handle) : nullptr;
	auto RigidHandle = bHasDynamicData ? static_cast<Chaos::FPBDRigidParticleHandle*>(Handle) : nullptr;
	const FDirtyChaosProperties& ParticleData = Dirty.PropertyData;
	FPBDRigidsEvolutionGBF& Evolution = *Solver.GetEvolution();
	
	const FParticleNonFrequentData* NewNonFrequentData = ParticleData.FindNonFrequentData(Manager, DataIdx);

#if UE_WITH_REMOTE_OBJECT_HANDLE
	bool bIsMigrated = false;
	if (FSerializedDataBufferPtr PendingStateData = Dirty.Proxy && !Dirty.Proxy->IsInitialized() ? Solver.GetSerializer().PopPendingInternalSerializedStateForProxy(Dirty.Proxy) : nullptr)
	{
		// Currently we don't serialize collision geometry and shape data, so we need to apply it first
		// This might change in the future
		if (NewNonFrequentData)
		{
			Handle->SetNonFrequentData(*NewNonFrequentData);
		}

		Solver.GetSerializer().ApplySerializedStateToParticle(Handle, *PendingStateData);

		bIsMigrated = true;
	}
#endif
	{
		if (bResimInitialized)	//todo: assumes particles are always initialized as enabled. This is not true in future versions of code, so check PushData
		{
			Evolution.EnableParticle(Handle);
		}
		// move the copied game thread data into the handle
		{
			auto NewXR = ParticleData.FindXR(Manager, DataIdx);
			auto NewDynamicMiscData = ParticleData.FindDynamicMisc(Manager, DataIdx);

			if(NewXR)
			{
				// @todo(chaos): we need to know if this is a teleport or not and pass that on. See UE-165746
				// For now we just set bIsTeleport to true since that's the no-impact option for SetParticleTransform
				// (there would be issues if we report a non-teleport move for an initial-position a long way from the origin)
				const bool bIsTeleport = true;
				bool bWakeUp = true;
				if (NewDynamicMiscData && NewDynamicMiscData->ObjectState() == EObjectStateType::Sleeping)
				{
					bWakeUp = false;
				}

				Evolution.SetParticleTransform(Handle, NewXR->X(), NewXR->R(), bIsTeleport, bWakeUp);
			}

#if UE_WITH_REMOTE_OBJECT_HANDLE
			if (!bIsMigrated)
#endif
			{
				if (NewNonFrequentData)
				{
					// Geometry may have changed, we need to remove the particle and its collisions from the graph
					Evolution.InvalidateParticle(Handle);

					Handle->SetNonFrequentData(*NewNonFrequentData);
				}
			}

			auto NewVelocities = bHasKinematicData ? ParticleData.FindVelocities(Manager, DataIdx) : nullptr;
			if(NewVelocities)
			{
				Evolution.SetParticleVelocities(KinematicHandle, NewVelocities->V(), NewVelocities->W());
			}
			
			auto NewKinematicTargetGT = bHasKinematicData ? ParticleData.FindKinematicTarget(Manager, DataIdx) : nullptr;
			bool bSkipDirtyParticle = false;
			if (NewKinematicTargetGT)
			{
				bSkipDirtyParticle = Evolution.SetParticleKinematicTarget(KinematicHandle, *NewKinematicTargetGT);
			}

			if(NewXR || NewNonFrequentData || NewVelocities || NewKinematicTargetGT)
			{
				// Update world-space cached state like the bounds
				// @todo(chaos): do we need to do this here? It should be done in Integrate and ApplyKinematicTarget so only really Statics need this...
				const bool bHasKinematicTarget = (NewKinematicTargetGT != nullptr) && (NewKinematicTargetGT->GetMode() == EKinematicTargetMode::Position);
				const FRigidTransform3 WorldTransform = !bHasKinematicTarget ? FRigidTransform3(Handle->GetX(), Handle->GetR()) : NewKinematicTargetGT->GetTransform();
				Handle->UpdateWorldSpaceState(WorldTransform, FVec3(0));

				// Kinematic target added the particle in the active list or in the moving kinematic list and so it will be updated
				// No need to added into the dirty list 
				if (!bSkipDirtyParticle)
				{
					Evolution.DirtyParticle(*Handle);
				}
				
			}

			if(bHasDynamicData)
			{
				if(const FParticleMassProps* NewData = ParticleData.FindMassProps(Manager,DataIdx))
				{
					Evolution.SetParticleMassProps(RigidHandle, *NewData);
				}

				if(const FParticleDynamics* NewData = ParticleData.FindDynamics(Manager, DataIdx))
				{
					Evolution.SetParticleDynamics(RigidHandle, *NewData);
				}

				if(NewDynamicMiscData )
				{
					Solver.SetParticleDynamicMisc(RigidHandle, *NewDynamicMiscData);
				}
			}
		}
	}

		//shape properties
		bool bUpdateCollisionData = false;
		bool bHasCollision = false;
		bool bHasMaterial = false;
		for(int32 ShapeDataIdx : Dirty.ShapeDataIndices)
		{
			const FShapeDirtyData& ShapeData = ShapesData[ShapeDataIdx];
			const int32 ShapeIdx = ShapeData.GetShapeIdx();

			if(auto NewData = ShapeData.FindCollisionData(Manager, ShapeDataIdx))
			{
				bUpdateCollisionData = true;
				Handle->ShapesArray()[ShapeIdx]->SetCollisionData(*NewData);

				const FCollisionData& CollisionData = Handle->ShapesArray()[ShapeIdx]->GetCollisionData();
				bHasCollision |= CollisionData.HasCollisionData();
			}
			if(auto NewData = ShapeData.FindMaterials(Manager, ShapeDataIdx))
			{
				Handle->ShapesArray()[ShapeIdx]->SetMaterialData(*NewData);
				bHasMaterial = true;
			}
		}
		
		// If the material, geometry, shape data, or sleep properties changed, we need to notify any systems that cache material data
		if (bHasMaterial || bUpdateCollisionData || NewNonFrequentData || bHasDynamicData)
		{
			Evolution.ParticleMaterialChanged(Handle);
		}

		if(bUpdateCollisionData && !ForceNoCollisionIntoSQ)
		{
			//Some shapes were not dirty and may have collision - so have to iterate them all. TODO: find a better way to handle this case
			if(!bHasCollision && Dirty.ShapeDataIndices.Num() != Handle->ShapesArray().Num())
			{
				for (const TUniquePtr<FPerShapeData>& Shape : Handle->ShapesArray())
				{
					const FCollisionData& CollisionData = Shape->GetCollisionData();
					bHasCollision |= CollisionData.HasCollisionData();

					if (bHasCollision) { break; }
				}
			}

			Handle->SetHasCollision(bHasCollision);

			if(bHasCollision)
			{
				// destroy collision constraints so that mid-phase is recreated with newly added shapes if any
				Evolution.DestroyTransientConstraints(Handle);
				//make sure it's in acceleration structure
				Evolution.DirtyParticle(*Handle);
			}
			else
			{
				Evolution.RemoveParticleFromAccelerationStructure(*Handle);
			}
		}
}

//
// TGeometryParticle<FReal, 3> template specialization 
//

void FSingleParticlePhysicsProxy::PushToPhysicsState(const Chaos::FDirtyPropertiesManager& Manager, int32 DataIdx, const Chaos::FDirtyProxy& Dirty, Chaos::FShapeDirtyData* ShapesData, Chaos::FReal ExternalDt)
{
	using namespace Chaos;
	FPBDRigidsSolver& RigidsSolver = *static_cast<FPBDRigidsSolver*>(Solver);
	const int32 CurFrame = RigidsSolver.GetCurrentFrame();
	const FRewindData* RewindData = RigidsSolver.GetRewindData();
	const bool bResimInitialized = RewindData && RewindData->IsResim() && CurFrame == InitializedOnStep;
	switch(Dirty.PropertyData.GetParticleBufferType())
	{
	case EParticleType::Static: PushToPhysicsStateImp<EParticleType::Static>(Manager, Handle, DataIdx, Dirty, ShapesData, RigidsSolver, bResimInitialized, ExternalDt); break;
	case EParticleType::Kinematic: PushToPhysicsStateImp<EParticleType::Kinematic>(Manager, Handle, DataIdx, Dirty, ShapesData, RigidsSolver, bResimInitialized, ExternalDt); break;
	case EParticleType::Rigid: PushToPhysicsStateImp<EParticleType::Rigid>(Manager, Handle, DataIdx, Dirty, ShapesData, RigidsSolver, bResimInitialized, ExternalDt); break;
	default: check(false); //unexpected path
	}
}

void FSingleParticlePhysicsProxy::ClearAccumulatedData()
{
	if(auto Rigid = Particle->CastToRigidParticle())
	{
		Rigid->ClearForces(false);
		Rigid->ClearTorques(false);
		Rigid->SetLinearImpulseVelocity(Chaos::FVec3(0), false);
		Rigid->SetAngularImpulseVelocity(Chaos::FVec3(0), false);
	}
	
	Particle->ClearDirtyFlags();
}

template <typename T>
void BufferPhysicsResultsImp(Chaos::FDirtyRigidParticleData& PullData, T* Particle)
{
	PullData.X = Particle->GetX();
	PullData.R = Particle->GetR();
	PullData.V = Particle->GetV();
	PullData.W = Particle->GetW();
	PullData.ObjectState = Particle->ObjectState();
}

void FSingleParticlePhysicsProxy::BufferPhysicsResults(Chaos::FDirtyRigidParticleData& PullData)
{
	using namespace Chaos;
	// Move simulation results into the double buffer.
	FPBDRigidParticleHandle* RigidHandle = Handle ? Handle->CastToRigidParticle() : nullptr;	//TODO: can handle be null?
	if(RigidHandle)
	{
		PullData.SetProxy(*this);
		BufferPhysicsResultsImp(PullData, RigidHandle);
	}
}

void FSingleParticlePhysicsProxy::BufferPhysicsResults_External(Chaos::FDirtyRigidParticleData& PullData)
{
	if(auto Rigid = Particle->CastToRigidParticle())
	{
		PullData.SetProxy(*this);
		BufferPhysicsResultsImp(PullData, Rigid);
	}
}

bool ShouldUpdateTransformFromSimulation(const Chaos::FPBDRigidParticle& Rigid)
{
	if (Rigid.ObjectState() == Chaos::EObjectStateType::Kinematic)
	{
		switch (Chaos::SyncKinematicOnGameThread)
		{
		case 0:
			return false;
		case 1:
			return true;
		default:
			return Rigid.UpdateKinematicFromSimulation();
		}
	}
	return true;
}

bool FSingleParticlePhysicsProxy::PullFromPhysicsState(const Chaos::FDirtyRigidParticleData& PullData,int32 SolverSyncTimestamp, const Chaos::FDirtyRigidParticleData* NextPullData, const Chaos::FRealSingle* Alpha, const FDirtyRigidParticleReplicationErrorData* Error, const Chaos::FReal AsyncFixedTimeStep)
{
	using namespace Chaos;

	// Move buffered data into the TPBDRigidParticle without triggering invalidation of the physics state.
	Chaos::FPBDRigidParticle* Rigid = Particle ? Particle->CastToRigidParticle() : nullptr;
	if (Rigid)
	{
		bool bNeedUpdateShapeBounds = bPullPhysicsStateForceUpdateBounds;
		// Note that kinematics should either be updated here (following simulation), or when the
		// kinematic target is set in FChaosEngineInterface::SetKinematicTarget_AssumesLocked If the
		// logic in one place is changed, it should be checked in the other place too.
		bool bUpdatePositionFromSimulation = ShouldUpdateTransformFromSimulation(*Rigid);
		const FSingleParticleProxyTimestamp* ProxyTimestamp = PullData.GetTimestamp();

		FProxyInterpolationBase* InterpData = GetInterpolationData();
		if (Error)
		{
			if (RenderInterpolationCVars::bRenderInterpErrorVelocityCorrection)
			{
				InterpData = GetOrCreateErrorInterpolationData<FProxyInterpolationErrorVelocity>();
			}
			else
			{
				InterpData = GetOrCreateErrorInterpolationData<FProxyInterpolationError>();
			}
			check(InterpData);
			

			const FReal ErrorDistanceSqr = Error->ErrorX.SizeSquared();
			FReal ErrorCorrectionDuration = RenderInterpolationCVars::RenderInterpErrorCorrectionDuration;
			FReal MaxErrorCorrection = RenderInterpolationCVars::RenderInterpMaximumErrorCorrectionBeforeSnapping;
			FReal MaxErrorDesyncTime = RenderInterpolationCVars::RenderInterpMaximumErrorCorrectionDesyncTimeBeforeSnapping;
			if (FErrorInterpolationSettings * ErrorSettings = InterpData->GetErrorInterpolationSettings())
			{
				ErrorCorrectionDuration = ErrorSettings->ErrorCorrectionDuration;
				MaxErrorCorrection = ErrorSettings->MaximumErrorCorrectionBeforeSnapping;
				MaxErrorDesyncTime = ErrorSettings->MaximumErrorCorrectionDesyncTimeBeforeSnapping;
			}
			MaxErrorCorrection = MaxErrorCorrection * MaxErrorCorrection; // Square the value

			auto VelocitySnapLimitHelper = [ErrorDistanceSqr, MaxErrorDesyncTime](const FVec3& Velocity) -> const bool
			{
				if (MaxErrorDesyncTime <= 0.0)
				{
					return false;
				}
				return ErrorDistanceSqr < (Velocity * MaxErrorDesyncTime).SizeSquared();
			};

			// Accumulate error if within interpolation limits
			if (ErrorDistanceSqr < MaxErrorCorrection || VelocitySnapLimitHelper(Rigid->V()) || VelocitySnapLimitHelper(PullData.V) || (NextPullData && VelocitySnapLimitHelper(NextPullData->V)))
			{
				const int32 RenderInterpErrorCorrectionDurationTicks = FMath::FloorToInt32(ErrorCorrectionDuration / AsyncFixedTimeStep); // Convert duration from seconds to simulation ticks
				InterpData->AccumulateErrorXR(Error->ErrorX, Error->ErrorR, SolverSyncTimestamp, RenderInterpErrorCorrectionDurationTicks);

				const int32 RenderInterpErrorVelocitySmoothingDurationTicks = FMath::FloorToInt32(RenderInterpolationCVars::RenderInterpErrorVelocitySmoothingDuration / AsyncFixedTimeStep); // Convert duration from seconds to simulation ticks
				InterpData->SetVelocitySmoothing(Rigid->V(), Rigid->X(), RenderInterpErrorVelocitySmoothingDurationTicks);
			}
		}
		const bool bHasInterpolationData = InterpData != nullptr;

		if (NextPullData)
		{
			const Chaos::FRealSingle LerpAlpha = RenderInterpolationCVars::bSnapToLatest ? 1.0f : *Alpha;

			auto LerpHelper = [SolverSyncTimestamp](const auto& Prev, const auto& OverwriteProperty) -> const auto*
			{
				//if overwrite is in the future, do nothing
				//if overwrite is on this step, we want to interpolate from overwrite to the result of the frame that consumed the overwrite
				//if overwrite is in the past, just do normal interpolation

				//this is nested because otherwise compiler can't figure out the type of nullptr with an auto return type
				return OverwriteProperty.Timestamp <= SolverSyncTimestamp ? (OverwriteProperty.Timestamp < SolverSyncTimestamp ? &Prev : &OverwriteProperty.Value) : nullptr;
			};

			if (bUpdatePositionFromSimulation)
			{
				if (bHasInterpolationData)
				{
					InterpData->UpdateError(SolverSyncTimestamp, AsyncFixedTimeStep);
				}

				const bool bIsReplicationErrorSmoothing = bHasInterpolationData ? InterpData->IsErrorSmoothing() : false;
				bool DirectionalDecayPerformed = false;

				if (const FVec3* Prev = LerpHelper(PullData.X, ProxyTimestamp->OverWriteX))
				{
					FVec3 Target = FMath::Lerp(*Prev, NextPullData->X, LerpAlpha);
					if (bIsReplicationErrorSmoothing)
					{
						DirectionalDecayPerformed = InterpData->DirectionalDecay(NextPullData->X - *Prev, RenderInterpolationCVars::RenderInterpErrorDirectionalDecayMultiplier);

						Target += InterpData->GetErrorX(LerpAlpha);

						if (InterpData->IsErrorVelocitySmoothing())
						{
#if CHAOS_DEBUG_DRAW
							if (RenderInterpolationCVars::bRenderInterpDebugDraw)
							{
								const FVector ZOffset = FVector(0,0, RenderInterpolationCVars::RenderInterpDebugDrawZOffset);
								Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow(ZOffset + (Target - InterpData->GetErrorX(LerpAlpha)), ZOffset + Target, 1, FColor::Blue, false, 5.0f, 0, 0.5f);
								Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow(ZOffset + InterpData->GetErrorVelocitySmoothingX(LerpAlpha), ZOffset + Target, 1, FColor::Blue, false, 5.0f, 0, 0.5f);
								Chaos::FDebugDrawQueue::GetInstance().DrawDebugBox(ZOffset + Target, FVector(2, 1, 1), Rigid->R(), FColor::Cyan, false, 5.f, 0, 0.25f);
							}
#endif // CHAOS_DEBUG_DRAW

							const int32 RenderInterpErrorVelocitySmoothingDurationTicks = FMath::FloorToInt32(RenderInterpolationCVars::RenderInterpErrorVelocitySmoothingDuration / AsyncFixedTimeStep); // Convert duration from seconds to simulation ticks
							Target = FMath::Lerp(Target, InterpData->GetErrorVelocitySmoothingX(LerpAlpha), InterpData->GetErrorVelocitySmoothingAlpha(RenderInterpErrorVelocitySmoothingDurationTicks));
						}
					}

					Rigid->SetX(Target, false);
					bNeedUpdateShapeBounds = true;
				}

				if (const FQuat* Prev = LerpHelper(PullData.R, ProxyTimestamp->OverWriteR))
				{
					FQuat Target = FMath::Lerp(*Prev, NextPullData->R, LerpAlpha);
					if (bIsReplicationErrorSmoothing)
					{
						Target = Target * InterpData->GetErrorR(LerpAlpha); // Add rotational error offset in local space
					}
					Rigid->SetR(Target, false);
					bNeedUpdateShapeBounds = true;
				}
				
#if CHAOS_DEBUG_DRAW
				if (RenderInterpolationCVars::bRenderInterpDebugDraw)
				{
					const FVector ZOffset = FVector(0, 0, RenderInterpolationCVars::RenderInterpDebugDrawZOffset);
					Chaos::FDebugDrawQueue::GetInstance().DrawDebugBox(ZOffset + NextPullData->X, FVector(2, 1, 1), NextPullData->R, FColor::Yellow, false, 5.f, 0, 0.5f);
					Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow(ZOffset + PullData.X, ZOffset + NextPullData->X, 0.5f, FColor::Yellow, false, 5.0f, 0, 0.5f);
					Chaos::FDebugDrawQueue::GetInstance().DrawDebugBox(ZOffset + Rigid->X(), FVector(2, 1, 1), Rigid->R(), DirectionalDecayPerformed ? FColor::Cyan : FColor::Green, false, 5.f, 0, 0.5f);

					if (bIsReplicationErrorSmoothing)
					{
						if (Error)
						{
							Chaos::FDebugDrawQueue::GetInstance().DrawDebugBox(ZOffset + PullData.X, FVector(4, 2, 2), PullData.R, FColor::Red, false, 5.f, 0, 0.5f);
							Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow(ZOffset + (PullData.X + Error->ErrorX), ZOffset + PullData.X, 1, FColor::Red, false, 5.0f, 0, 0.5f);
						}

						if (InterpData->IsErrorVelocitySmoothing())
						{
							Chaos::FDebugDrawQueue::GetInstance().DrawDebugBox(ZOffset + InterpData->GetErrorVelocitySmoothingX(LerpAlpha), FVector(2, 2, 2), Rigid->R(), FColor::Purple, false, 5.f, 0, 0.5f);
						}
						else
						{
							Chaos::FDebugDrawQueue::GetInstance().DrawDebugDirectionalArrow(ZOffset + (Rigid->X() - InterpData->GetErrorX(LerpAlpha)), ZOffset + Rigid->X(), 1, FColor::Blue, false, 5.0f, 0, 0.5f);
						}
					}
				}
#endif // CHAOS_DEBUG_DRAW
			}

			if (const FVec3* Prev = LerpHelper(PullData.V, ProxyTimestamp->OverWriteV))
			{
				FVec3 Target = FMath::Lerp(*Prev, NextPullData->V, LerpAlpha);
				Rigid->SetV(Target, false);
			}

			if (const FVec3* Prev = LerpHelper(PullData.W, ProxyTimestamp->OverWriteW))
			{
				FVec3 Target = FMath::Lerp(*Prev, NextPullData->W, LerpAlpha);
				Rigid->SetW(Target, false);
			}

			//we are interpolating from PullData to Next, but the timestamp is associated with Next
			//since we are interpolating it means we must have not seen Next yet, so the timestamp has to be strictly less than
			if (ProxyTimestamp->ObjectStateTimestamp < SolverSyncTimestamp)
			{
				Rigid->SetObjectState(PullData.ObjectState, true, /*bInvalidate=*/false);
			}
			else if(ProxyTimestamp->ObjectStateTimestamp == SolverSyncTimestamp && LerpAlpha == 1.f)
			{
				//if timestamp is the same as next, AND alpha is exactly 1, we are exactly at Next's time
				//so we can use its sleep state
				Rigid->SetObjectState(NextPullData->ObjectState, true, /*bInvalidate=*/false);
			}
		}
		else
		{
			if (bUpdatePositionFromSimulation)
			{
				//no interpolation, just ignore if overwrite comes after
				if (SolverSyncTimestamp >= ProxyTimestamp->OverWriteX.Timestamp)
				{
					Rigid->SetX(PullData.X, false);
					bNeedUpdateShapeBounds = true;
				}

				if (SolverSyncTimestamp >= ProxyTimestamp->OverWriteR.Timestamp)
				{
					Rigid->SetR(PullData.R, false);
					bNeedUpdateShapeBounds = true;
				}
			}

			if(SolverSyncTimestamp >= ProxyTimestamp->OverWriteV.Timestamp)
			{
				Rigid->SetV(PullData.V, false);
			}

			if(SolverSyncTimestamp >= ProxyTimestamp->OverWriteW.Timestamp)
			{
				Rigid->SetW(PullData.W, false);
			}

			if (SolverSyncTimestamp >= ProxyTimestamp->ObjectStateTimestamp)
			{
				Rigid->SetObjectState(PullData.ObjectState, true, /*bInvalidate=*/false);
			}
		}
		
		if (bNeedUpdateShapeBounds)
		{
			Rigid->UpdateShapeBounds();
		}
	}
	return true;
}

bool FSingleParticlePhysicsProxy::IsDirty()
{
	return Particle->IsDirty();
}

Chaos::EWakeEventEntry FSingleParticlePhysicsProxy::GetWakeEvent() const
{
	//question: should this API exist on proxy?
	auto Rigid = Particle->CastToRigidParticle();
	return Rigid ? Rigid->GetWakeEvent() : Chaos::EWakeEventEntry::None;
}

void FSingleParticlePhysicsProxy::ClearEvents()
{
	//question: should this API exist on proxy?
	if(auto Rigid = Particle->CastToRigidParticle())
	{
		Rigid->ClearEvents();
	}
}

void FRigidBodyHandle_External::SetShapeFilterData(int32 InShapeIndex, const Filter::FShapeFilterData& FilterData)
{
	VerifyContext();
	GetParticle_LowLevel()->SetShapeFilterData(InShapeIndex, FilterData);
}

void FRigidBodyHandle_External::SetShapeFilterInstanceData(int32 InShapeIndex, const Filter::FInstanceData& InstanceData)
{
	VerifyContext();
	GetParticle_LowLevel()->SetShapeFilterInstanceData(InShapeIndex, InstanceData);
}

}
