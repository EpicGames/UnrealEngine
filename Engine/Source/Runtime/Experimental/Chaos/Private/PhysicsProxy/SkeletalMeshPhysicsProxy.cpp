// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsProxy/SkeletalMeshPhysicsProxy.h"
#include "GeometryCollection/GeometryCollectionCollisionStructureManager.h"
#include "ChaosStats.h"
#include "PhysicsSolver.h"

// @todo(chaos): remove this file

FSkeletalMeshPhysicsProxy::FSkeletalMeshPhysicsProxy(UObject* InOwner, const FInitFunc& InInitFunc)
	: Base(InOwner)
{
}

FSkeletalMeshPhysicsProxy::~FSkeletalMeshPhysicsProxy()
{
}

void FSkeletalMeshPhysicsProxy::Initialize()
{
}

void FSkeletalMeshPhysicsProxy::Reset()
{
}

bool FSkeletalMeshPhysicsProxy::IsSimulating() const
{
	return false;
}

void FSkeletalMeshPhysicsProxy::UpdateKinematicBodiesCallback(const FParticlesType& Particles, const float Dt, const float Time, FKinematicProxy& Proxy)
{
}

void FSkeletalMeshPhysicsProxy::StartFrameCallback(const float InDt, const float InTime)
{
}

void FSkeletalMeshPhysicsProxy::EndFrameCallback(const float InDt)
{
}

void FSkeletalMeshPhysicsProxy::CreateRigidBodyCallback(FParticlesType& Particles)
{
}

void FSkeletalMeshPhysicsProxy::ParameterUpdateCallback(FParticlesType& InParticles, const float InTime)
{}

void FSkeletalMeshPhysicsProxy::DisableCollisionsCallback(TSet<TTuple<int32, int32>>& InPairs)
{}

void FSkeletalMeshPhysicsProxy::AddForceCallback(FParticlesType& InParticles, const float InDt, const int32 InIndex)
{}

void FSkeletalMeshPhysicsProxy::BindParticleCallbackMapping(Chaos::TArrayCollectionArray<PhysicsProxyWrapper>& PhysicsProxyReverseMap, Chaos::TArrayCollectionArray<int32>& ParticleIDReverseMap)
{
}

void FSkeletalMeshPhysicsProxy::SyncBeforeDestroy() 
{}

void FSkeletalMeshPhysicsProxy::OnRemoveFromScene()
{
}

void FSkeletalMeshPhysicsProxy::BufferPhysicsResults()
{
}

void FSkeletalMeshPhysicsProxy::FlipBuffer()
{
}

bool FSkeletalMeshPhysicsProxy::PullFromPhysicsState(const int32 SolverSyncTimestamp)
{
	return true;
}

void FSkeletalMeshPhysicsProxy::CaptureInputs(const float Dt, const FInputFunc& InputFunc)
{
}
