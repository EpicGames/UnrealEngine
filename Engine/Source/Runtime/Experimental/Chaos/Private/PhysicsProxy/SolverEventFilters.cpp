// Copyright Epic Games, Inc. All Rights Reserved.

#include "SolverEventFilters.h"
#include "Chaos/CollisionResolutionTypes.h"
#include "Chaos/ExternalCollisionData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SolverEventFilters)

namespace Chaos
{

	bool FSolverCollisionEventFilter::Pass(const FCollidingData& InData) const
	{
		const FReal MinVelocitySquared = FMath::Square(Settings.MinSpeed);
		const FReal MinImpulseSquared = FMath::Square(Settings.MinImpulse);

		if (Settings.MinMass > 0.0f && InData.Mass1 < Settings.MinMass && InData.Mass2 < Settings.MinMass)
			return false;

		if (Settings.MinSpeed > 0.0f && InData.Velocity1.SizeSquared() < MinVelocitySquared && InData.Velocity2.SizeSquared() < MinVelocitySquared)
			return false;

		if (Settings.MinImpulse > 0.0f && InData.AccumulatedImpulse.SizeSquared() < MinImpulseSquared)
			return false;

		return true;
	}

	bool FSolverTrailingEventFilter::Pass(const FTrailingData& InData) const
	{
		const FReal MinSpeedThresholdSquared = Settings.MinSpeed * Settings.MinSpeed;

		if (Settings.MinMass > 0.0f && InData.Mass < Settings.MinMass)
			return false;

		if (Settings.MinSpeed > 0 && InData.Velocity.SizeSquared() < MinSpeedThresholdSquared)
			return false;

		if (Settings.MinVolume > 0)
		{
			FVec3 Extents = InData.BoundingBox.Extents();
			FReal Volume = Extents[0] * Extents[1] * Extents[2];

			if (Volume < Settings.MinVolume)
				return false;
		}

		return true;
	}

	bool FSolverBreakingEventFilter::Pass(const FBreakingData& InData) const
	{
		const FReal MinSpeedThresholdSquared = Settings.MinSpeed * Settings.MinSpeed;

		if (Settings.MinMass > 0.0f && InData.Mass < Settings.MinMass)
			return false;

		if (Settings.MinSpeed > 0 && InData.Velocity.SizeSquared() < MinSpeedThresholdSquared)
			return false;

		if (Settings.MinVolume > 0)
		{
			FVec3 Extents = InData.BoundingBox.Extents();
			FReal Volume = Extents[0] * Extents[1] * Extents[2];

			if (Volume < Settings.MinVolume)
				return false;
		}

		return true;
	}

	bool FSolverRemovalEventFilter::Pass(const FRemovalData& InData) const
	{
		if (Settings.MinMass > 0.0f && InData.Mass < Settings.MinMass)
			return false;

		if (Settings.MinVolume > 0)
		{
			FVec3 Extents = InData.BoundingBox.Extents();
			FReal Volume = Extents[0] * Extents[1] * Extents[2];

			if (Volume < Settings.MinVolume)
				return false;
		}

		return true;
	}

} // namespace Chaos

