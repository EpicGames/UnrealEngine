// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsProxy/StaticMeshPhysicsProxy.h"
#include "PhysicsSolver.h"
#include "GeometryCollection/GeometryCollectionCollisionStructureManager.h"
#include "Chaos/Box.h"
#include "Chaos/Capsule.h"
#include "Chaos/Sphere.h"
#include "Chaos/ErrorReporter.h"
#include "Chaos/Serializable.h"
#include "ChaosStats.h"

// @todo(chaos): remove this file

FStaticMeshPhysicsProxy::FStaticMeshPhysicsProxy(UObject* InOwner, FCallbackInitFunc InInitFunc, FSyncDynamicFunc InSyncFunc)
	: Base(InOwner)
{
}

void FStaticMeshPhysicsProxy::Initialize()
{
}

void FStaticMeshPhysicsProxy::Reset()
{
}

void FStaticMeshPhysicsProxy::BufferKinematicUpdate(const FPhysicsProxyKinematicUpdate& InParamUpdate)
{
};

bool FStaticMeshPhysicsProxy::IsSimulating() const
{
	return false;
}

void FStaticMeshPhysicsProxy::UpdateKinematicBodiesCallback(const FParticlesType& Particles, const float Dt, const float Time, FKinematicProxy& Proxy)
{
}

void FStaticMeshPhysicsProxy::StartFrameCallback(const float InDt, const float InTime)
{

}

void FStaticMeshPhysicsProxy::EndFrameCallback(const float InDt)
{
}

void FStaticMeshPhysicsProxy::BindParticleCallbackMapping(Chaos::TArrayCollectionArray<PhysicsProxyWrapper> & PhysicsProxyReverseMap, Chaos::TArrayCollectionArray<int32> & ParticleIDReverseMap)
{
}

void FStaticMeshPhysicsProxy::CreateRigidBodyCallback(FParticlesType& Particles)
{
}

void FStaticMeshPhysicsProxy::ParameterUpdateCallback(FParticlesType& InParticles, const float InTime)
{
}

void FStaticMeshPhysicsProxy::DisableCollisionsCallback(TSet<TTuple<int32, int32>>& InPairs)
{
}

void FStaticMeshPhysicsProxy::AddForceCallback(FParticlesType& InParticles, const float InDt, const int32 InIndex)
{
}

void FStaticMeshPhysicsProxy::OnRemoveFromScene()
{
}

void FStaticMeshPhysicsProxy::BufferPhysicsResults()
{
}

void FStaticMeshPhysicsProxy::FlipBuffer()
{
}

bool FStaticMeshPhysicsProxy::PullFromPhysicsState(const int32 SolverSyncTimestamp)
{
	return true;
}
