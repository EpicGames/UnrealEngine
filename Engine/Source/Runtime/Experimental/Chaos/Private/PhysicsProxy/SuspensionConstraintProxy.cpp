// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsProxy/SuspensionConstraintProxy.h"

#include "ChaosStats.h"
#include "Chaos/Collision/SpatialAccelerationBroadPhase.h"
#include "Chaos/Collision/CollisionConstraintFlags.h"
#include "Chaos/ErrorReporter.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/GeometryParticles.h"
#include "Chaos/Serializable.h"
#include "Chaos/PBDSuspensionConstraints.h"
#include "Chaos/Framework/MultiBufferResource.h"
#include "Chaos/PhysicsObjectInternal.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "PhysicsSolver.h"

namespace Chaos
{

FSuspensionConstraintPhysicsProxy::FSuspensionConstraintPhysicsProxy(FSuspensionConstraint* InConstraint, FPBDSuspensionConstraintHandle* InHandle, UObject* InOwner)
	: Base(EPhysicsProxyType::SuspensionConstraintType, InOwner, MakeShared<FProxyTimestampBase>())
	, Constraint_GT(InConstraint)
	, Constraint_PT(InHandle)
	, bInitialized(false)
{
	check(Constraint_GT!=nullptr);
	Constraint_GT->SetProxy(this);
}

TGeometryParticleHandle<FReal, 3>*
FSuspensionConstraintPhysicsProxy::GetParticleHandleFromProxy(IPhysicsProxyBase* ProxyBase)
{
	if (ProxyBase)
	{
		if (ProxyBase->GetType() == EPhysicsProxyType::SingleParticleProxy)
		{
			return ((FSingleParticlePhysicsProxy*)ProxyBase)->GetHandle_LowLevel();
		}
	}
	return nullptr;
}

void FSuspensionConstraintPhysicsProxy::InitializeOnPhysicsThread(FPBDRigidsSolver* InSolver, FDirtyPropertiesManager& Manager, int32 DataIdx, FDirtyChaosProperties& RemoteData)
{
	auto& Handles = InSolver->GetParticles().GetParticleHandles();
	if (Handles.Size())
	{
		auto& SuspensionConstraints = InSolver->GetSuspensionConstraints();
		if (const FPhysicsObjectProperty* Body = RemoteData.FindSuspensionPhysicsObject(Manager, DataIdx))
		{
			FGeometryParticleHandle* Handle = Body->PhysicsBody->GetRootParticle<Chaos::EThreadContext::Internal>();

			if (Handle)
			{
				const FPBDSuspensionSettings* SuspensionSettings = RemoteData.FindSuspensionSettings(Manager, DataIdx);
				const FSuspensionLocation* SuspensionLocation = RemoteData.FindSuspensionLocation(Manager, DataIdx);

				if (SuspensionSettings && SuspensionLocation)
				{
					Constraint_PT = SuspensionConstraints.AddConstraint(Handle, SuspensionLocation->Location, *SuspensionSettings);
					Handle->AddConstraintHandle(Constraint_PT);
				}
			}
		}
	}
}


void FSuspensionConstraintPhysicsProxy::PushStateOnGameThread(FDirtyPropertiesManager& Manager, int32 DataIdx, FDirtyChaosProperties& RemoteData)
{
	if (Constraint_GT && Constraint_GT->IsValid())
	{
		Constraint_GT->SyncRemoteData(Manager, DataIdx, RemoteData);
	}
}


void FSuspensionConstraintPhysicsProxy::PushStateOnPhysicsThread(FPBDRigidsSolver* InSolver, const FDirtyPropertiesManager& Manager, int32 DataIdx, const FDirtyChaosProperties& RemoteData)
{
	if (Constraint_PT && Constraint_PT->IsValid())
	{
		if (const FPBDSuspensionSettings* Data = RemoteData.FindSuspensionSettings(Manager, DataIdx))
		{
			Constraint_PT->GetSettings() = *Data;
		}
	}
}


void FSuspensionConstraintPhysicsProxy::DestroyOnGameThread()
{
	if (Constraint_GT)
	{
		delete Constraint_GT; 
		Constraint_GT = nullptr;
	}
}

void FSuspensionConstraintPhysicsProxy::DestroyOnPhysicsThread(FPBDRigidsSolver* RBDSolver)
{
	if (Constraint_PT)
	{
		// @todo(chaos): clean up constraint management
		RBDSolver->GetEvolution()->RemoveConstraintFromConstraintGraph(Constraint_PT);
		auto& SuspensionConstraints = RBDSolver->GetSuspensionConstraints();
		SuspensionConstraints.RemoveConstraint(Constraint_PT->GetConstraintIndex());
		Constraint_PT = nullptr;
	}
}

void FSuspensionConstraintPhysicsProxy::UpdateTargetOnPhysicsThread(FPBDRigidsSolver* RBDSolver, const FVector& TargetPos, const IPhysicsProxyBase* GroundProxy, const FVector& Normal, bool Enabled)
{
	if (Constraint_PT && Constraint_PT->IsValid())
	{
		auto& SuspensionConstraints = RBDSolver->GetSuspensionConstraints();
		SuspensionConstraints.GetSettings(Constraint_PT->GetConstraintIndex()).Target = TargetPos;
		SuspensionConstraints.GetSettings(Constraint_PT->GetConstraintIndex()).Normal = Normal;
		SuspensionConstraints.GetSettings(Constraint_PT->GetConstraintIndex()).Enabled = Enabled;

		FGeometryParticleHandle* GroundParticle = nullptr;
		if (GroundProxy)
		{
			if (GroundProxy->GetType() == EPhysicsProxyType::SingleParticleProxy)
			{
				GroundParticle = ((FSingleParticlePhysicsProxy*)GroundProxy)->GetHandle_LowLevel();
			}
		}

		SuspensionConstraints.SetGroundParticle(Constraint_PT->GetConstraintIndex(), GroundParticle);
	}
}

}