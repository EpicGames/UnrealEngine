// Copyright Epic Games, Inc. All Rights Reserved.

#include "RewindData.h"
#include "PBDRigidsSolver.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "PhysicsProxy/ClusterUnionPhysicsProxy.h"
#include "PhysicsProxy/GeometryCollectionPhysicsProxy.h"
#include "HAL/IConsoleManager.h"
#include "Chaos/Island/IslandManager.h"
#include "Chaos/ResimConsoleVariables.h"
#include "ChaosVisualDebugger/ChaosVisualDebuggerTrace.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/StringOutputDevice.h"

namespace Chaos
{
	namespace ResimConsoleVars
	{
		bool bEnableRewindLogs = false;
		static FAutoConsoleVariableRef CVarEnableRewindLogs(
			TEXT("p.Chaos.EnableRewindLogs"),
			bEnableRewindLogs,
			TEXT("Set to true to log important rewind logs"));

		int32 BubbleResimFreezeMode = 1;
		static FAutoConsoleVariableRef CVarBubbleResimFreezeMode(
			TEXT("p.Chaos.BubbleResim.FreezeMode"),
			BubbleResimFreezeMode,
			TEXT("Mode used to freeze particles outside of resim bubble. Mode 0: Make particles static. Mode 1: Make particles kinematic."));

		bool BubbleResimEnableCheapFreeze = false;
		static FAutoConsoleVariableRef CVarBubbleResimEnableCheapFreeze(
			TEXT("p.Chaos.BubbleResim.EnableCheapFreeze"),
			BubbleResimEnableCheapFreeze,
			TEXT("Use batched SetObjectState version instead of going through every single particles to modify the object state, to freeze/unfreeze a particle"));

		float BubbleResimExtraSweptDistance = 10.0f;
		static FAutoConsoleVariableRef CVarBubbleResimExtraSweptDistance(
			TEXT("p.Chaos.BubbleResim.ExtraSweptDistanceForCorrectedParticle"),
			BubbleResimExtraSweptDistance,
			TEXT("Extra swept distance when forming resim bubbles"));

		float BubbleResimExtraThickness = 10.0f;
		static FAutoConsoleVariableRef CVarBubbleResimExtraThickness(
			TEXT("p.Chaos.BubbleResim.ExtraThicknessForCorrectedParticle"),
			BubbleResimExtraThickness,
			TEXT("Extra AABB thickness when forming resim bubbles"));

		int32 BubbleResimGrowthMaxNumTries = 3;
		static FAutoConsoleVariableRef CVarBubbleResimGrowthMaxNumTries(
			TEXT("p.Chaos.BubbleResim.MaxNumTriesForResimBubbleGrowth"),
			BubbleResimGrowthMaxNumTries,
			TEXT("When forming bubbles, we grow the bubble by the AABBs of particles it overlaps. This is the maximum number of such growths we allow."));

		bool bIncludeKinematicParticlesInBubbleGrowth = false;
		static FAutoConsoleVariableRef CVarIncludeKinematicParticlesInBubbleGrowth(
			TEXT("p.Chaos.BubbleResim.IncludeKinematicParticlesInBubbleGrowth"),
			bIncludeKinematicParticlesInBubbleGrowth,
			TEXT("If true, kinematic particles overlapping resim bubbles cause the bubble to grow. Kinematic particles overlapping resim bubbles will always participate to the resim, regardless whether this is enabled or not."));
	}
	
	namespace ResimDefines
	{
		const FName BubbleResimDebugDrawTag(TEXT("BubbleResimDebugDraws"));
	}

FVec3 FGeometryParticleState::ZeroVector = FVec3(0);

void FGeometryParticleStateBase::SyncSimWritablePropsFromSim(FDirtyPropData Manager,const TPBDRigidParticleHandle<FReal,3>& Rigid)
{
	FDirtyChaosPropertyFlags Flags;
	Flags.MarkDirty(EChaosPropertyFlags::XR);
	Flags.MarkDirty(EChaosPropertyFlags::Velocities);
	Flags.MarkDirty(EChaosPropertyFlags::DynamicMisc);
	FDirtyChaosProperties Dirty;
	Dirty.SetFlags(Flags);

#if 0
	ParticlePositionRotation.SyncRemoteData(Manager,Dirty,[&Rigid](auto& Data)
	{
		Data.CopyFrom(Rigid);
	});

	Velocities.SyncRemoteData(Manager,Dirty,[&Rigid](auto& Data)
	{
		Data.SetV(Rigid.PreV());
		Data.SetW(Rigid.PreW());
	});

	KinematicTarget.SyncRemoteData(Manager, Dirty, [&Rigid](auto& Data)
	{
		Data = Rigid.KinematicTarget();
	});

	DynamicsMisc.SyncRemoteData(Manager, Dirty, [&Rigid](auto& Data)
	{
		Data.CopyFrom(Rigid);
		Data.SetObjectState(Rigid.PreObjectState());	//everything else is not writable by sim so must be the same
	});
#endif
}

void FGeometryParticleStateBase::SyncDirtyDynamics(FDirtyPropData& DestManager,const FDirtyChaosProperties& Dirty,const FConstDirtyPropData& SrcManager)
{
#if 0
	FParticleDirtyData DirtyFlags;
	DirtyFlags.SetFlags(Dirty.GetFlags());

	Dynamics.SyncRemoteData(DestManager,DirtyFlags,[&Dirty,&SrcManager](auto& Data)
	{
		Data = Dirty.GetDynamics(*SrcManager.Ptr,SrcManager.DataIdx);
	});
#endif
}

bool SimWritablePropsMayChange(const TGeometryParticleHandle<FReal,3>& Handle)
{
	const auto ObjectState = Handle.ObjectState();
	return ObjectState == EObjectStateType::Dynamic || ObjectState == EObjectStateType::Sleeping;
}

namespace ResimulationCVars
{
	float DesyncLeniencyValue = 0.5f;
	FAutoConsoleVariableRef CVarResimDesyncLeniencyValue(TEXT("p.Resim.DesyncLeniencyValue"), DesyncLeniencyValue, TEXT("Value of leniency when comparing current state with cached state to determine if particle is in sync of not, leniency should cover network quantization of state else particles will always be considered desynced after having a server state applied. Default RepMovement quantization is WholeNumber which means a max loss of 0.5 units in precision."));
}

template <bool bSkipDynamics>
bool FGeometryParticleStateBase::IsInSync(const FGeometryParticleHandle& Handle, const FFrameAndPhase FrameAndPhase, const FDirtyPropertiesPool& Pool) const
{
	if (!ParticlePositionRotation.IsInSync(Handle, FrameAndPhase, Pool))
	{
		return false;
	}

	if (!NonFrequentData.IsInSync(Handle, FrameAndPhase, Pool))
	{
		return false;
	}

	//todo: deal with state change mismatch

	if (auto Kinematic = Handle.CastToKinematicParticle())
	{
		if (!Velocities.IsInSync(*Kinematic, FrameAndPhase, Pool))
		{
			return false;
		}

		if (!KinematicTarget.IsInSync(*Kinematic, FrameAndPhase, Pool))
		{
			return false;
		}
	}

	if (auto Rigid = Handle.CastToRigidParticle())
	{
		if (!bSkipDynamics)
		{
			if (!Dynamics.IsInSync(*Rigid, FrameAndPhase, Pool))
			{
				return false;
			}
		}

		if (!DynamicsMisc.IsInSync(*Rigid, FrameAndPhase, Pool))
		{
			return false;
		}

		if (!MassProps.IsInSync(*Rigid, FrameAndPhase, Pool))
		{
			return false;
		}
	}
	
	//TODO: this assumes geometry is never modified. Geometry modification has various issues in higher up Chaos code. Need stable shape id
	//For now iterate over all the shapes in latest and see if they have any mismatches
	/*if(ShapesArrayState.PerShapeData.Num())
	{
		return false;	//if any shapes changed just resim, this is not efficient but at least it's correct
	}*/

	return true;
}

template <bool bSkipDynamics>
bool FJointStateBase::IsInSync(const FPBDJointConstraintHandle& Handle, const FFrameAndPhase FrameAndPhase, const FDirtyPropertiesPool& Pool) const
{
	if (!JointSettings.IsInSync(Handle, FrameAndPhase, Pool))
	{
		return false;
	}

	return true;
}


bool bCVarRewindDataOptimization = true;
FAutoConsoleVariableRef CVarRewindDataOptimization(TEXT("p.Resim.RewindDataOptimization"), bCVarRewindDataOptimization, TEXT("Default value for RewinData optimization, note that this can be overridden at runtime by API calls. Effect: Only alter the minimum required properties during a resim for particles not marked for FullResim and only cache data during the PrePushData phase and lower memory allocation for the history cache to 1/3 of non-optimized flow."));

static int32 CalculateNumFrames(FPBDRigidsSolver* InSolver, int32 NumFrames)
{
	if (ensure(InSolver) && NumFrames <= 0 && InSolver->GetAsyncDeltaTime() > UE_SMALL_NUMBER)
	{
		NumFrames = FMath::Max<int32>(1, FMath::CeilToInt32((0.001f * FPBDRigidsSolver::GetPhysicsHistoryTimeLength()) / InSolver->GetAsyncDeltaTime()));
	}
	return NumFrames;
}

FRewindData::FRewindData(FPBDRigidsSolver* InSolver, int32 NumFrames, bool InRewindDataOptimization, int32 InCurrentFrame)
	: Managers(CalculateNumFrames(InSolver, NumFrames) + 1)
	, Solver(InSolver)
	, CurFrame(InCurrentFrame)
	, LatestFrame(InCurrentFrame)
	, FramesSaved(0)
	, DataIdxOffset(0)
	, bNeedsSave(false)
	, bRewindDataOptimization(InRewindDataOptimization)
	, LatestTargetFrame(0)
	, AllowResizingFromTickTrate(false)
{
	if (NumFrames <= 0 && Solver)
	{
		AllowResizingFromTickTrate = true;
		InitializedDeltaTime = Solver->GetAsyncDeltaTime();
	}

	ResimClampedFrames = Managers.Capacity();
	RegisterEvolutionCallbacks();
}

FRewindData::FRewindData(FPBDRigidsSolver* InSolver, int32 NumFrames, int32 InCurrentFrame)
	: Managers(CalculateNumFrames(InSolver, NumFrames) + 1)
	, Solver(InSolver)
	, CurFrame(InCurrentFrame)
	, LatestFrame(InCurrentFrame)
	, FramesSaved(0)
	, DataIdxOffset(0)
	, bNeedsSave(false)
	, bRewindDataOptimization(bCVarRewindDataOptimization)
	, LatestTargetFrame(0)
	, AllowResizingFromTickTrate(false)
{
	if (NumFrames <= 0 && Solver)
	{
		AllowResizingFromTickTrate = true;
		InitializedDeltaTime = Solver->GetAsyncDeltaTime();
	}

	ResimClampedFrames = Managers.Capacity();
	RegisterEvolutionCallbacks();
}

void FRewindData::Init(FPBDRigidsSolver* InSolver, int32 NumFrames, bool InRewindDataOptimization, int32 InCurrentFrame)
{
	Solver = InSolver;
	CurFrame = InCurrentFrame;
	LatestFrame = InCurrentFrame;
	bRewindDataOptimization = InRewindDataOptimization;
	LatestTargetFrame = 0;
	AllowResizingFromTickTrate = false;

	if (NumFrames <= 0 && Solver)
	{
		AllowResizingFromTickTrate = true;
		InitializedDeltaTime = Solver->GetAsyncDeltaTime();
	}

	Managers = TCircularBuffer<FFrameManagerInfo>(CalculateNumFrames(InSolver, NumFrames) + 1);
	ResimClampedFrames = Managers.Capacity();

	RegisterEvolutionCallbacks();
}

void FRewindData::Init(FPBDRigidsSolver* InSolver, int32 NumFrames, int32 InCurrentFrame)
{
	Solver = InSolver;
	CurFrame = InCurrentFrame;
	LatestFrame = InCurrentFrame;
	LatestTargetFrame = 0;
	AllowResizingFromTickTrate = false;

	if (NumFrames <= 0 && Solver)
	{
		AllowResizingFromTickTrate = true;
		InitializedDeltaTime = Solver->GetAsyncDeltaTime();
	}

	Managers = TCircularBuffer<FFrameManagerInfo>(CalculateNumFrames(InSolver, NumFrames) + 1);
	ResimClampedFrames = Managers.Capacity();

	RegisterEvolutionCallbacks();
}

void FRewindData::RegisterEvolutionCallbacks()
{
	if (!ensureMsgf(Solver, TEXT("FRewindData::RegisterEvolutionCallbacks No Solver")))
	{
		return;
	}

	if (!ensureMsgf(Solver->GetEvolution(), TEXT("FRewindData::RegisterEvolutionCallbacks No Evolution")))
	{
		return;
	}

	Solver->GetEvolution()->SetCaptureRewindKinematicTargetFunction([this](const TParticleView<TPBDRigidParticles<FReal, 3>>& ActiveKinematicParticles)
		{
			ProcessDirtyKinematicTargets(ActiveKinematicParticles);
		});

	Solver->GetEvolution()->SetCaptureRewindDataFunction([this](const TParticleView<TPBDRigidParticles<FReal, 3>>& ActiveParticles)
		{
			ProcessDirtyPTParticles(ActiveParticles);
		});
}

void FRewindData::ApplyInputs(const int32 ApplyFrame, const bool bResetSolver)
{
	for (TWeakPtr<FBaseRewindHistory>& InputHistory : InputHistories)
	{
		if (InputHistory.IsValid())
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
				InputHistory.Pin().Get()->ApplyInputs(ApplyFrame, bResetSolver);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}
}
void FRewindData::RewindStates(const int32 RewindFrame, const bool bResetSolver)
{
	for (TWeakPtr<FBaseRewindHistory>& StateHistory : StateHistories)
	{
		if (StateHistory.IsValid())
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
				StateHistory.Pin().Get()->RewindStates(RewindFrame, bResetSolver);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}
}

void FRewindData::ApplyTargets(const int32 Frame, const bool bResetSimulation)
{
	QUICK_SCOPE_CYCLE_COUNTER(ChaosRewindAndResim_ApplyTargets);
	CVD_SCOPE_TRACE_SOLVER_STEP(CVDDC_ResimDetails, TEXT("Rewind Apply Targets"));

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	RewindStates(Frame, bResetSimulation);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	EnsureIsInPhysicsThreadContext();

	//If property changed between Frame and CurFrame, record the latest value and rewind to old
	FFrameAndPhase RewindFrameAndPhase{ Frame, FFrameAndPhase::PrePushData };

	auto RewindHelper = [RewindFrameAndPhase, this](auto Obj, bool bResimAsFollower, auto& Property, const auto& RewindFunc)
	{
		if (!Property.IsClean(RewindFrameAndPhase) && !bResimAsFollower)
		{
			RewindFunc(Obj, *Property.Read(RewindFrameAndPhase, PropertiesPool));
		}
	};

	DynamicHandlesUpdateHolder.Reset(0);
	SleepingHandlesUpdateHolder.Reset(0);
	StaticHandlesUpdateHolder.Reset(0);
	KinematicHandlesUpdateHolder.Reset(0);

	for (FDirtyParticleInfo& DirtyParticleInfo : DirtyParticles)
	{
		FGeometryParticleHandle* PTParticle = DirtyParticleInfo.GetObjectPtr();
		FGeometryParticleStateBase& History = DirtyParticleInfo.GetHistory();

		// In bubble resim we skip applying targets to any particle marked as frozen during resim
		if (ResimConsoleVars::bIsBubbleResimEnabled && PTParticle->ResimType() == EResimType::FrozenDuringResim)
		{
			continue;
		}

		const bool bResimAsFollower = DirtyParticleInfo.bResimAsFollower;

		RewindHelper(PTParticle, bResimAsFollower, History.TargetPositions, [](auto Particle, const auto& Data)
			{
				Particle->SetXR(Data);
			});
		RewindHelper(PTParticle->CastToKinematicParticle(), bResimAsFollower, History.TargetVelocities, [](auto Particle, const auto& Data)
			{
				Particle->SetV(Data.V());
				Particle->SetW(Data.W());
			});
		RewindHelper(PTParticle->CastToRigidParticle(), bResimAsFollower, History.TargetStates, [this](auto Particle, const auto& Data)
		{ 
			if (Particle == nullptr || Solver->GetEvolution() == nullptr)
			{
				return;
			}

			// Enable or disable the particle
			if (Particle->Disabled() != Data.Disabled())
			{
				if (Data.Disabled())
				{
					Solver->GetEvolution()->DisableParticle(Particle);
				}
				else
				{
					Solver->GetEvolution()->EnableParticle(Particle);
				}
			}

			// If we changed kinematics we need to rebuild the inertia conditioning
			const bool bDirtyInertiaConditioning = (Particle->ObjectState() != Data.ObjectState());
			if (bDirtyInertiaConditioning)
			{
				Particle->SetInertiaConditioningDirty();
			}

			switch (Data.ObjectState())
			{
				case EObjectStateType::Dynamic:
				{
					DynamicHandlesUpdateHolder.Add(Particle);
					break;
				}
				case EObjectStateType::Sleeping:
				{
					SleepingHandlesUpdateHolder.Add(Particle);
					break;
				}
				case EObjectStateType::Static:
				{
					StaticHandlesUpdateHolder.Add(Particle);
					break;
				}
				case EObjectStateType::Kinematic:
				{
					KinematicHandlesUpdateHolder.Add(Particle);
					break;
				}
			}
		});

		if (FPBDRigidsEvolution* Evolution = Solver->GetEvolution())
		{
			QUICK_SCOPE_CYCLE_COUNTER(ChaosRewindAndResim_ApplyTargets_UpdateObjectsState);

			// Update the particles object states
			Evolution->SetParticlesObjectState(DynamicHandlesUpdateHolder, EObjectStateType::Dynamic, false);
			Evolution->SetParticlesObjectState(SleepingHandlesUpdateHolder, EObjectStateType::Sleeping, false);
			Evolution->SetParticlesObjectState(StaticHandlesUpdateHolder, EObjectStateType::Static, false);
			Evolution->SetParticlesObjectState(KinematicHandlesUpdateHolder, EObjectStateType::Kinematic, false);

			// Update the dirty views
			Evolution->GetParticles().UpdateDirtyViews();
		}

#if WITH_CHAOS_VISUAL_DEBUGGER
		if (FChaosVisualDebuggerTrace::IsTracing() && CVDDC_ResimDetails->IsChannelEnabled())
		{
			CVD_TRACE_PARTICLE(PTParticle);
		}
#endif

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if (!History.TargetPositions.IsClean(RewindFrameAndPhase) && Chaos::FPhysicsSolverBase::CanDebugNetworkPhysicsPrediction())
		{
			UE_LOGF(LogChaos, Log, "Reset particle %d position to the target %ls at frame %d", PTParticle->UniqueIdx().Idx, *PTParticle->GetX().ToString(), Frame);
		}
#endif
	}
}

const int32 FRewindData::CompareTargetsToLastFrame()
{
	int32 RewindFrame = INDEX_NONE;
	const FFrameAndPhase FrameAndPhase{ CurrentFrame() - 1, FFrameAndPhase::PrePushData };

	if (LatestTargetFrame < FrameAndPhase.Frame)
	{
		// Early out if we only have targets earlier than the previous simulated frame
		// NOTE: This is the normal flow, we should only run this logic when the client is desynced behind the server and we receive targets from the server ahead of time.
		return RewindFrame;
	}

	// TODO: Take per actor settings into consideration via NetworkPhysicsSettingsComponent
	const bool bCompareX = Chaos::FPhysicsSolverBase::GetResimulationErrorPositionThresholdEnabled();
	const bool bCompareR = Chaos::FPhysicsSolverBase::GetResimulationErrorRotationThresholdEnabled();
	const bool bCompareV = Chaos::FPhysicsSolverBase::GetResimulationErrorLinearVelocityThresholdEnabled();
	const bool bCompareW = Chaos::FPhysicsSolverBase::GetResimulationErrorAngularVelocityThresholdEnabled();

	bool ShouldTriggerResim = false;

	// Iterate over targets that exist for current frame
	for (FDirtyParticleInfo& DirtyParticleInfo : DirtyParticles)
	{
		// TODO: Only iterate source target states, i.e. states that are not predicted/interpolated to fill in gaps
		
		FGeometryParticleStateBase& History = DirtyParticleInfo.GetHistory();
		if ((bCompareX || bCompareR) && !History.TargetPositions.IsEmpty())
		{
			// Compare with particle for this frame and mark resim if needed from CurrentFrame()
			if (const FParticlePositionRotation* TargetState = History.TargetPositions.Read(FrameAndPhase, PropertiesPool))
			{
				if (const FParticlePositionRotation* PastState = History.ParticlePositionRotation.Read(FrameAndPhase, PropertiesPool))
				{
					if (bCompareX)
					{
						ShouldTriggerResim |= FRewindData::CheckVectorThreshold(TargetState->GetX(), PastState->GetX(), FPhysicsSolverBase::GetResimulationErrorPositionThreshold()); // TODO: Take per actor settings into consideration via NetworkPhysicsSettingsComponent
					}

					if (bCompareR && !ShouldTriggerResim)
					{
						ShouldTriggerResim |= FRewindData::CheckQuaternionThreshold(TargetState->GetR(), PastState->GetR(), FPhysicsSolverBase::GetResimulationErrorRotationThreshold()); // TODO: Take per actor settings into consideration via NetworkPhysicsSettingsComponent
					}
				}
			}
		}

		if (!ShouldTriggerResim && (bCompareV || bCompareW) && !History.TargetVelocities.IsEmpty())
		{
			// Compare with particle for this frame and mark resim if needed from CurrentFrame()
			if (const FParticleVelocities* TargetState = History.TargetVelocities.Read(FrameAndPhase, PropertiesPool))
			{
				if (const FParticleVelocities* PastState = History.Velocities.Read(FrameAndPhase, PropertiesPool))
				{
					if (bCompareV)
					{
						ShouldTriggerResim |= FRewindData::CheckVectorThreshold(TargetState->GetV(), PastState->GetV(), FPhysicsSolverBase::GetResimulationErrorLinearVelocityThreshold()); // TODO: Take per actor settings into consideration via NetworkPhysicsSettingsComponent
					}

					if (bCompareW && !ShouldTriggerResim)
					{
						ShouldTriggerResim |= FRewindData::CheckVectorThreshold(TargetState->GetW(), PastState->GetW(), FPhysicsSolverBase::GetResimulationErrorAngularVelocityThreshold()); // TODO: Take per actor settings into consideration via NetworkPhysicsSettingsComponent
					}
				}
			}
		}
	}

	if (ShouldTriggerResim)
	{
		RewindFrame = FrameAndPhase.Frame;
	}

	return RewindFrame;
}

bool FRewindData::CheckVectorThreshold(FVec3 A, FVec3 B, float Threshold)
{
	const FVector Delta = A - B;
	return Delta.SizeSquared() >= (Threshold * Threshold);
}

bool FRewindData::CheckQuaternionThreshold(FQuat A, FQuat B, float ThresholdDegrees)
{
	// We assume A and B are normalized
	const double ShortestArcAngle = FMath::Abs(FMath::Clamp(A | B, -1.0f, 1.0f));
	const double AngleRadians = 2.0f * FMath::Acos(ShortestArcAngle);
	const double AngleDegrees = FMath::RadiansToDegrees(AngleRadians);

	return AngleDegrees >= ThresholdDegrees;
}

bool FRewindData::OverlapsAnyResimBubble(const Chaos::FAABB3& InAABB) const
{
	for (const FAABB3& ResimBubble : ResimBubbles)
	{
		if (ResimBubble.Intersects(InAABB))
		{
			return true;
		}
	}

	return false;
}

FAABB3 FRewindData::GetParticleSweptAABB(const FGeometryParticleHandle* ParticleHandle, FRewindData::FDirtyParticleInfo* DirtyParticleInfo, const FFrameAndPhase& AtFrameAndPhase, float HorizonDuration, float ExtraSweptDistance, float ExtraThickness) const
{
	FAABB3 ResultAABB = FAABB3::ZeroAABB();
	const FGeometryParticleHandle* GeometryParticle = DirtyParticleInfo ? DirtyParticleInfo->GetObjectPtr() : ParticleHandle;
	if (GeometryParticle)
	{
		FVector ParticlePosition = FVector::ZeroVector;
		FQuat ParticleRotation = FQuat::Identity;
		FVector ParticleVelocity = FVector::ZeroVector;
		bool HistoryFound = false;
		bool VelocityHistoryFound = false;
		if (DirtyParticleInfo)
		{
			FGeometryParticleStateBase& History = DirtyParticleInfo->GetHistory();
			const FParticlePositionRotation* PositionRotation = History.TargetPositions.Read(AtFrameAndPhase, PropertiesPool);
			PositionRotation = PositionRotation ? PositionRotation : History.ParticlePositionRotation.Read(AtFrameAndPhase, PropertiesPool);
			if (PositionRotation)
			{
				ParticlePosition = PositionRotation->GetX();
				ParticleRotation = PositionRotation->GetR();
				HistoryFound = true;
			}
			const FParticleVelocities* Velocities = History.TargetVelocities.Read(AtFrameAndPhase, PropertiesPool);
			Velocities = Velocities ? Velocities : History.Velocities.Read(AtFrameAndPhase, PropertiesPool);
			if (Velocities)
			{
				ParticleVelocity = Velocities->GetV();
				VelocityHistoryFound = true;
			}
		}
		if (!HistoryFound)
		{
			ParticlePosition = GeometryParticle->GetX();
			ParticleRotation = GeometryParticle->GetR();
		}
		if (!VelocityHistoryFound)
		{
			if (const FKinematicGeometryParticleHandle* KinematicParticle = GeometryParticle->CastToKinematicParticle())
			{
				ParticleVelocity = KinematicParticle->GetV();
			}
		}
		ResultAABB = GeometryParticle->LocalBounds().TransformedAABB(FTransform(ParticleRotation, ParticlePosition));
		ResultAABB.Thicken(ExtraThickness);
		FVector ParticleVelocityNormal = ParticleVelocity;
		if (ParticleVelocityNormal.Normalize())
		{
			ResultAABB.GrowByVector(HorizonDuration * ParticleVelocity + ExtraSweptDistance * ParticleVelocityNormal);
		}
	}
	return ResultAABB;
}

struct FNonStaticUniqueOverlapVisitor : public ISpatialVisitor<FAccelerationStructureHandle, FReal>
{
	using FVisitorFunctionType = TFunction<bool(const FGeometryParticleHandle*)>;

	FNonStaticUniqueOverlapVisitor(FVisitorFunctionType InParticleVisitor) : ParticleVisitor(InParticleVisitor) {}
	virtual bool Overlap(const TSpatialVisitorData<FAccelerationStructureHandle>& Instance) override
	{
		if (FGeometryParticleHandle* Particle = Instance.Payload.GetGeometryParticleHandle_PhysicsThread())
		{
			bool WasParticleAlreadyInSet = false;
			if (Particle->ObjectState() !=  EObjectStateType::Static)
			{
				OverlappingParticles.Add(Particle, &WasParticleAlreadyInSet);
				if (!WasParticleAlreadyInSet)
				{
					return ParticleVisitor(Particle);
				}
			}
		}
		return true; // Keep going
	}
	virtual bool Raycast(const TSpatialVisitorData<FAccelerationStructureHandle>& Instance, FQueryFastData& CurData) override { return false; }
	virtual bool Sweep(const TSpatialVisitorData<FAccelerationStructureHandle>& Instance, FQueryFastData& CurData) override { return false; }

	TSet<FGeometryParticleHandle*> OverlappingParticles;
	FVisitorFunctionType ParticleVisitor;
};

void FRewindData::FindParticlesOverlappingResimBubbles(const FFrameAndPhase& RewindFrameAndPhase, float TotalResimDuration, TSet<FGeometryParticleHandle*>& OutParticlesOverlappingResimBubbles)
{
	if (!ResimConsoleVars::bIsBubbleResimEnabled)
	{
		return;
	}
	
	FPBDRigidsEvolution* Evolution = Solver->GetEvolution();

	// First seed resim bubbles by building swept AABBs around particles needing resim
	{
		QUICK_SCOPE_CYCLE_COUNTER(ChaosRewindAndResim_SeedResimBubbles);
		for (FDirtyParticleInfo& DirtyParticleInfo : DirtyParticles)
		{
			if (DirtyParticleInfo.bNeedsResim)
			{
				FGeometryParticleHandle* ParticleNeedingResim = DirtyParticleInfo.GetObjectPtr();
				FAABB3 ResimAABB = GetParticleSweptAABB(DirtyParticleInfo.GetObjectPtr(), &DirtyParticleInfo, RewindFrameAndPhase,
					TotalResimDuration, Chaos::ResimConsoleVars::BubbleResimExtraSweptDistance, Chaos::ResimConsoleVars::BubbleResimExtraThickness);
				//static float ResimAABBThickening = 1.0f;
				//ResimAABB.Thicken(ResimAABBThickening);
				ResimBubbles.Add(ResimAABB);
				// Important: it's better to add the particles needing resim for sure to ParticlesOverlappingBubble,
				// because they might not be found to overlap the bubble based on the spatial acceleration structure.
				// The spatial acceleration we have is for the frame we rewind from, not the frame we rewind to
				OutParticlesOverlappingResimBubbles.Add(ParticleNeedingResim);
				CVD_TRACE_DEBUG_DRAW_BOX(FBox(ResimAABB.Min(), ResimAABB.Max()), ResimDefines::BubbleResimDebugDrawTag, FColor(255,255,191)/*Pale Yellow*/);
			}
		}
	}

	// Then query the spatial acceleration structure to determine which particles overlap resim bubbles, and grow the bubble accordingly
	// We repeat this until we don't overlap anything new, or we have reached the maximum number of tries
	if (ISpatialAccelerationCollection<FAccelerationStructureHandle, FReal, 3>*SpatialAcceleration = Evolution->GetSpatialAcceleration())
	{
		QUICK_SCOPE_CYCLE_COUNTER(ChaosRewindAndResim_GrowResimBubbles);
		for (FAABB3& ResimAABB : ResimBubbles)
		{
			int32 NumTries = 0;
			int32 MaxNumTries = Chaos::ResimConsoleVars::BubbleResimGrowthMaxNumTries;
			bool FoundNewOverlaps = true;
			// This lambda causes the bubble to grow when a particle is found to overlap
			// It will be called only once for every new particle found overlapping the resim bubble as it grows
			auto LambdaParticleVisitor = [&](const FGeometryParticleHandle* Particle)->bool
				{
					FDirtyParticleInfo* DirtyParticleInfo = DirtyParticles.Find(Particle);
					FAABB3 ParticleResimAABB = GetParticleSweptAABB(Particle, DirtyParticleInfo, RewindFrameAndPhase,
						TotalResimDuration, Chaos::ResimConsoleVars::BubbleResimExtraSweptDistance, Chaos::ResimConsoleVars::BubbleResimExtraThickness);
					bool bIsParticleKinematic = (Particle->ObjectState() == EObjectStateType::Kinematic);
					bool bIncludeInBubbleGrowth = !bIsParticleKinematic || Chaos::ResimConsoleVars::bIncludeKinematicParticlesInBubbleGrowth;
					if (bIncludeInBubbleGrowth)
					{
						ResimAABB.GrowToInclude(ParticleResimAABB);
						FoundNewOverlaps = true;
					}
#if WITH_CHAOS_VISUAL_DEBUGGER
					float Alpha = float(NumTries) / float(MaxNumTries <= 1 ? 1 : MaxNumTries - 1);
					FLinearColor OverlapDrawColor = FMath::Lerp(FLinearColor(FColor::Magenta), FLinearColor(FColor::Purple), Alpha);
					CVD_TRACE_DEBUG_DRAW_BOX(FBox(ParticleResimAABB.Min(), ParticleResimAABB.Max()), ResimDefines::BubbleResimDebugDrawTag, OverlapDrawColor.ToFColor(true));
#endif // WITH_CHAOS_VISUAL_DEBUGGER
					return true; // Keep going
				};
			FNonStaticUniqueOverlapVisitor OverlapVisitor(LambdaParticleVisitor);
			while (FoundNewOverlaps && NumTries < MaxNumTries)
			{
				FoundNewOverlaps = false;
				SpatialAcceleration->Overlap(ResimAABB, OverlapVisitor);
#if WITH_CHAOS_VISUAL_DEBUGGER
				float Alpha = float(NumTries) / float(MaxNumTries <= 1 ? 1 : MaxNumTries - 1);
				FLinearColor BubbleDrawColor = FMath::Lerp(FLinearColor::Yellow, FLinearColor::Red, Alpha);
				CVD_TRACE_DEBUG_DRAW_BOX(FBox(ResimAABB.Min(), ResimAABB.Max()), ResimDefines::BubbleResimDebugDrawTag, BubbleDrawColor.ToFColor(true));
#endif // WITH_CHAOS_VISUAL_DEBUGGER
				NumTries++;
			}
			OutParticlesOverlappingResimBubbles.Append(OverlapVisitor.OverlappingParticles);
		}
	}
}

void FRewindData::FreezeParticlesOutsideResimBubbles()
{
	if (!ResimConsoleVars::bIsBubbleResimEnabled)
	{
		return;
	}

	QUICK_SCOPE_CYCLE_COUNTER(ChaosRewindAndResim_FreezeNonResimmingParticles);

	FPBDRigidsEvolution* Evolution = Solver->GetEvolution();

	TArray<FPBDRigidParticleHandle*, TInlineAllocator<64>> ParticlesToFreezeInResim;
	ParticlesToFreezeInResim.Reserve(64);

	for (auto& Particle : Evolution->GetParticles().GetActiveDynamicMovingKinematicParticlesView())
	{
		if (FGeometryParticleHandle* ParticleHandle = Particle.Handle())
		{
			if (FPBDRigidParticleHandle* RigidParticleHandle = ParticleHandle->CastToRigidParticle();
				RigidParticleHandle
				&& (RigidParticleHandle->ResimType() == EResimType::FrozenDuringResim)
				&& (RigidParticleHandle->ObjectState() != EObjectStateType::Static))
			{
				ParticlesToFreezeInResim.Add(RigidParticleHandle);
			}
		}
	}

	FrozenParticles.Reserve(ParticlesToFreezeInResim.Num());

	TArray<FPBDRigidParticleHandle*> ParticleHandles;
	if (Chaos::ResimConsoleVars::BubbleResimEnableCheapFreeze)
	{
		ParticleHandles.Reserve(ParticlesToFreezeInResim.Num());
	}
	const EObjectStateType FrozenObjectState = (ResimConsoleVars::BubbleResimFreezeMode == 0) ? EObjectStateType::Static : EObjectStateType::Kinematic;
	for (int32 ParticleToFreezeIndex = 0; ParticleToFreezeIndex < ParticlesToFreezeInResim.Num(); ++ParticleToFreezeIndex)
	{
		FPBDRigidParticleHandle* RigidParticleHandle = ParticlesToFreezeInResim[ParticleToFreezeIndex];
		if (IPhysicsProxyBase* PhysicsProxy = RigidParticleHandle->PhysicsProxy())
		{
			FrozenParticles.Add({PhysicsProxy, RigidParticleHandle->ObjectState()});

			if (Chaos::ResimConsoleVars::BubbleResimEnableCheapFreeze)
			{
				ParticleHandles.Add(RigidParticleHandle);
			}
			else
			{
				// This is too slow, e.g. it calls UpdateViews every time, but it might be more correct
				// TODO: extract the minimum of what's needed from SetParticleObjectState
				Evolution->SetParticleObjectState(RigidParticleHandle, FrozenObjectState); 
			}
		}
	}
	if (Chaos::ResimConsoleVars::BubbleResimEnableCheapFreeze)
	{
		QUICK_SCOPE_CYCLE_COUNTER(ChaosRewindAndResim_UpdateObjectsState);
		TArrayView<FPBDRigidParticleHandle*> DynamicParticles(ParticleHandles.GetData(), ParticleHandles.Num());
		Evolution->SetParticlesObjectState(DynamicParticles, FrozenObjectState, true);
	}
}

void FRewindData::UnfreezeParticlesOutsideResimBubbles()
{
	if (!ResimConsoleVars::bIsBubbleResimEnabled)
	{
		return;
	}
	
	FPBDRigidsEvolution* Evolution = Solver->GetEvolution();
	
	// Revert to pre-resim object state
	{
		QUICK_SCOPE_CYCLE_COUNTER(ChaosRewindAndResim_RevertObjectState);
		
		if(Chaos::ResimConsoleVars::BubbleResimEnableCheapFreeze)
		{
			DynamicHandlesUpdateHolder.Reset(FrozenParticles.Num());
			SleepingHandlesUpdateHolder.Reset(FrozenParticles.Num());
			StaticHandlesUpdateHolder.Reset(FrozenParticles.Num());
			KinematicHandlesUpdateHolder.Reset(FrozenParticles.Num());
		}
		for (TPair<Chaos::FPhysicsProxyHandle, EObjectStateType>& FrozenParticle : FrozenParticles)
		{
			if (IPhysicsProxyBase* Proxy = FrozenParticle.Key.Get())
			{
				EObjectStateType PreResimObjectState = FrozenParticle.Value;
				FSingleParticlePhysicsProxy* SingleParticlePhysicsProxy = static_cast<FSingleParticlePhysicsProxy*>(Proxy);
				if (FPBDRigidParticleHandle* RigidParticleHandle = SingleParticlePhysicsProxy->GetHandle_LowLevel()->CastToRigidParticle())
				{
					if (RigidParticleHandle->ObjectState() != PreResimObjectState)
					{
						if (Chaos::ResimConsoleVars::BubbleResimEnableCheapFreeze)
						{
							if (PreResimObjectState == EObjectStateType::Dynamic)
							{
								DynamicHandlesUpdateHolder.Add(RigidParticleHandle);
							}
							else if(PreResimObjectState == EObjectStateType::Sleeping)
							{
								SleepingHandlesUpdateHolder.Add(RigidParticleHandle);
							}
							if (PreResimObjectState == EObjectStateType::Static)
							{
								StaticHandlesUpdateHolder.Add(RigidParticleHandle);
							}
							else if (PreResimObjectState == EObjectStateType::Kinematic)
							{
								KinematicHandlesUpdateHolder.Add(RigidParticleHandle);
							}
						}
						else
						{
							// This is too expensive as it calls UpdateViews for each particle
							Evolution->SetParticleObjectState(RigidParticleHandle, PreResimObjectState);
						}
					}
#if WITH_CHAOS_VISUAL_DEBUGGER
					if (FChaosVisualDebuggerTrace::IsTracing() && CVDDC_ResimDetails->IsChannelEnabled())
					{
						// Trace modified particles. Unfortunately there is no way currently to only trace their object state so we trace the whole particle.
						CVD_TRACE_PARTICLE(RigidParticleHandle);
					}
#endif
				}
			}
		}
	}
	if (Chaos::ResimConsoleVars::BubbleResimEnableCheapFreeze)
	{
		QUICK_SCOPE_CYCLE_COUNTER(ChaosRewindAndResim_ReportObjectsState);
		
		// Update the frozen particles handles with the value pre-resim
		Evolution->SetParticlesObjectState(DynamicHandlesUpdateHolder, EObjectStateType::Dynamic, false);
		Evolution->SetParticlesObjectState(SleepingHandlesUpdateHolder, EObjectStateType::Sleeping, false);
		Evolution->SetParticlesObjectState(StaticHandlesUpdateHolder, EObjectStateType::Static, false);
		Evolution->SetParticlesObjectState(KinematicHandlesUpdateHolder, EObjectStateType::Kinematic, false);
	
		// Update the dirty views
		Evolution->GetParticles().UpdateDirtyViews();
	}
	{
		QUICK_SCOPE_CYCLE_COUNTER(ChaosRewindAndResim_MarkAllFrozenInResimByDefault);
		// Mark all particles as frozen in resim (default value)
		// Improve this by only going through particles that were unfrozen for resim
		for (auto& Particle : Evolution->GetParticles().GetActiveDynamicMovingKinematicParticlesView())
		{
			Particle.SetResimType(EResimType::FrozenDuringResim);
		}
	}
	// Empty frozen particles so we can start from scratch on the next rewind/resims
	FrozenParticles.Empty();
}

bool bForceSkipResim = false;
FAutoConsoleVariableRef CVarForceSkipResim(TEXT("p.Resim.ForceSkipResim"), bForceSkipResim, TEXT("Don't perform resimulations while this is true"));

bool bGuardFromPerformanceCollapse = true;
FAutoConsoleVariableRef CVarResimGuardFromPerformanceCollapse(TEXT("p.Resim.GuardFromPerformanceCollapse"), bGuardFromPerformanceCollapse, TEXT("Don't perform another resimulation before as much realtime has passed as last resim took."));

bool FRewindData::AllowRewindToFrame(int32 RewindFrame) const
{
	// Don't perform a resimulation if we are currently under cooldown from a previous resimulation
	if (LatestFrame < ResimCooldownUntilFrame)
	{
		return false;
	}

	// Stop resim from causing a performance collapse by taking longer to calculate than physics has moved forward in realtime
	if (bGuardFromPerformanceCollapse && ResimPerformanceCooldownCycleCounter > FPlatformTime::Cycles64())
	{
		return false;
	}

	return true;
}

bool FRewindData::CanRewindToFrame(int32 RewindFrame)
{
	auto ClearResimHelper = [&]()
		{
			// Clear ResimFrame since we will never be able to resim to this frame
			SetResimFrame(INDEX_NONE);
			if (Solver && Solver->GetEvolution())
			{
				Solver->GetEvolution()->GetIslandManager().ResetParticleResimFrame();
			}
		};

	if (bForceSkipResim)
	{
		ClearResimHelper();
		return false;
	}

	// We can't rewind forward
	if (RewindFrame > LatestFrame)
	{
		if (Chaos::ResimConsoleVars::bEnableRewindLogs)
		{
			UE_LOGF(LogChaos, Log, "CLIENT | PT | CanRewindToFrame | Failed due to rewind frame being later than latest simulated frame | Rewind Frame: %d | Latest Frame: %d", RewindFrame, LatestFrame);
		}

		ClearResimHelper();
		return false;
	}

	// We need marshaling manager data for the number of frames back we want to resim
	const int32 NumResimFrames = LatestFrame - RewindFrame;
	if (Solver && NumResimFrames > Solver->GetMarshallingManager().GetNumHistory_Internal())
	{
		if (Chaos::ResimConsoleVars::bEnableRewindLogs)
		{
			UE_LOGF(LogChaos, Log, "CLIENT | PT | CanRewindToFrame | Failed due to NumResimFrames is larger than the cached data in marshaling manager | Rewind Frame: %d | Resim Frames: %d | Marshaling History Count:: %d", RewindFrame, NumResimFrames, Solver->GetMarshallingManager().GetNumHistory_Internal());
		}

		ClearResimHelper();
		return false;
	}

	// We can't rewind outside of the rewind history
	const int32 EarliestFrame = GetEarliestFrame_Internal();
	if (RewindFrame < EarliestFrame)
	{
		if (Chaos::ResimConsoleVars::bEnableRewindLogs)
		{
			UE_LOGF(LogChaos, Log, "CLIENT | PT | CanRewindToFrame | Failed due to rewind frame earlier than available history | Rewind Frame: %d | Earliest Frame: %d", RewindFrame, EarliestFrame);
		}

		ClearResimHelper();
		return false;
	}

	// If we need to save and we are right on the edge of the buffer, we can't go back to earliest frame
	if (RewindFrame == EarliestFrame && bNeedsSave && FramesSaved == Managers.Capacity())
	{
		if (Chaos::ResimConsoleVars::bEnableRewindLogs)
		{
			UE_LOGF(LogChaos, Log, "CLIENT | PT | CanRewindToFrame | Failed due to rewinding to last available frame and bNeedsSave is set to true");
		}

		ClearResimHelper();
		return false;
	}

	return true;
}

CHAOS_API bool bResimAllowRewindToResimulatedFrames = true;
FAutoConsoleVariableRef CVarResimAllowRewindToResimulatedFrames(TEXT("p.Resim.AllowRewindToResimulatedFrames"), bResimAllowRewindToResimulatedFrames, TEXT("Allow rewinding back to a frame that was previously part of a resimulation. If a resimulation is performed between frame 100-110, allow a new resim from 105-115 if needed, else next resim will be able to start from frame 111."));

CHAOS_API float ResimAllowRewindToResimulatedFramesOverlapPercent = 1.0f;
FAutoConsoleVariableRef CVarResimAllowRewindToResimulatedFramesOverlapPercent(TEXT("p.Resim.AllowRewindToResimulatedFrames.OverlapPercent"), ResimAllowRewindToResimulatedFramesOverlapPercent, TEXT("Value in percent as a multiplier, 0.3 = 30%. When p.Resim.AllowRewindToResimulatedFrames is true, this sets how many frames of overlap we allow when rewinding back past a previous resimulation. If a resimulation is performed between frame 100-110 (10 frames), 30% overlap would allow 3 frames to get resimulated again, i.e. we would be allowed to rewind back to frame 107 again if we need to."));

int32 ResimCooldownFrames = 0;
FAutoConsoleVariableRef CVarResimCooldownFrames(TEXT("p.Resim.CooldownFrames"), ResimCooldownFrames, TEXT("Number of frames that needs to pass between each resimulation."));

void FRewindData::RewindToFrame(int32 RewindFrame)
{
	FPBDRigidsEvolution* Evolution = Solver ? Solver->GetEvolution() : nullptr;
	if (!Evolution)
	{
		return;
	}

	CVD_SCOPE_TRACE_SOLVER_STEP(CVDDC_ResimDetails, TEXT("Rewind To Frame"));

	// Start measuring resim time
	ResimStartCycleCounter = FPlatformTime::Cycles64();

	QUICK_SCOPE_CYCLE_COUNTER(RewindToFrame);

	EnsureIsInPhysicsThreadContext();

	if (Chaos::ResimConsoleVars::bEnableRewindLogs)
	{
		UE_LOGF(LogChaos, Log, "CLIENT | PT | FRewindData::RewindToFrame | Rewinding from Frame %d to Frame %d", CurFrame, RewindFrame);
	}

	//If property changed between Frame and CurFrame, record the latest value and rewind to old
	FFrameAndPhase RewindFrameAndPhase{ RewindFrame, FFrameAndPhase::PrePushData };
	FFrameAndPhase CurFrameAndPhase{ CurFrame, FFrameAndPhase::PrePushData };

	if (bResimAllowRewindToResimulatedFrames)
	{
		float NumFrames = static_cast<float>(CurFrame - RewindFrame);
		NumFrames *= ResimAllowRewindToResimulatedFramesOverlapPercent;
		BlockResimFrame = CurFrame - FMath::CeilToInt(NumFrames);
	}
	else
	{
		BlockResimFrame = CurFrame;
	}

	SetResimFrame(RewindFrame);
	ResimCooldownUntilFrame = LatestFrame + FMath::Max(ResimCooldownFrames, 0);

	CurFrame = RewindFrame;
	bNeedsSave = false;

	// This is used for bubble resim
	int32 NumResimFrames = CurFrameAndPhase.Frame - RewindFrameAndPhase.Frame;
	float TotalResimDuration = static_cast<float>(NumResimFrames * (Solver ? Solver->GetAsyncDeltaTime() : 0.0));

	// Bubble resim: form resim bubbles and determine particles overlapping them
	// This does nothing if bubble resim is disabled
	TSet<FGeometryParticleHandle*> ParticlesOverlappingBubble;
	FindParticlesOverlappingResimBubbles(RewindFrameAndPhase, TotalResimDuration, /*out param*/ParticlesOverlappingBubble);

	auto RewindHelper = [RewindFrameAndPhase, CurFrameAndPhase, this](auto Obj, bool bResimAsFollower, auto& Property, const auto& RewindFunc) -> bool
		{
			if (bResimAsFollower)
			{
				//If we're rewinding a particle that doesn't need to save head (resim as follower never checks for desync so we don't care about head)
				if (auto Val = Property.Read(RewindFrameAndPhase, PropertiesPool))
				{
					RewindFunc(Obj, *Val);
					return true;
				}
			}
			else
			{
				//If we're rewinding an object that needs to save head (during resim when we get back to latest frame and phase we need to check for desync)
				if (!Property.IsClean(RewindFrameAndPhase))
				{
					if (!bRewindDataOptimization)
					{
						// When not using optimized RewindData cache the current state in Phase::PrePushData on rewind.
						CopyDataFromObject(Property.WriteAccessMonotonic(CurFrameAndPhase, PropertiesPool), *Obj);
					}
					RewindFunc(Obj, *Property.Read(RewindFrameAndPhase, PropertiesPool));
					return true;
				}
			}
			return false;
		};

	// Determine resim type
	if (ResimConsoleVars::bIsBubbleResimEnabled)
	{
		// Bubble resim: Everything in the resim bubble fully resimulates unless marked for forced ResimAsFollower
		QUICK_SCOPE_CYCLE_COUNTER(ChaosRewindAndResim_MarkAllParticlesInBubbleFullResim);
		for (FDirtyParticleInfo& DirtyParticleInfo : DirtyParticles)
		{
			DirtyParticleInfo.bResimAsFollower = ParticlesToForceResimAsFollower.Contains(DirtyParticleInfo.GetObjectPtr());
		}

		for (FGeometryParticleHandle* GeometryParticle : ParticlesOverlappingBubble)
		{
			if (GeometryParticle)
			{
				if (FDirtyParticleInfo* DirtyParticleInfo = FindDirtyObj(*GeometryParticle))
				{
					if (!DirtyParticleInfo->bResimAsFollower)
					{
						const IPhysicsProxyBase* PhysicsProxy = GeometryParticle->PhysicsProxy();
						if (PhysicsProxy && PhysicsProxy->GetMarkedDeleted())
						{
							// Force resim as follower when pending delete and we have history for the particle (we have FDirtyParticleInfo)
							DirtyParticleInfo->bResimAsFollower = true;
						}
					}
					GeometryParticle->SetResimType(DirtyParticleInfo->bResimAsFollower ? EResimType::ResimAsFollower : EResimType::FullResim);
				}
				else
				{
					GeometryParticle->SetResimType(ParticlesToForceResimAsFollower.Contains(GeometryParticle) ? EResimType::ResimAsFollower : EResimType::FullResim);
				}
			}
		}
	}
	else
	{
		for (FDirtyParticleInfo& DirtyParticleInfo : DirtyParticles)
		{
			FGeometryParticleHandle* GeometryParticle = DirtyParticleInfo.GetObjectPtr();
			DirtyParticleInfo.bResimAsFollower = GeometryParticle->ObjectState() != EObjectStateType::Dynamic && GeometryParticle->ObjectState() != EObjectStateType::Sleeping;

			if (!DirtyParticleInfo.bResimAsFollower)
			{
				// Force resim as follower
				DirtyParticleInfo.bResimAsFollower = ParticlesToForceResimAsFollower.Contains(GeometryParticle);
			}

			if (!DirtyParticleInfo.bResimAsFollower)
			{
				const IPhysicsProxyBase* PhysicsProxy = GeometryParticle->PhysicsProxy();
				if (PhysicsProxy && PhysicsProxy->GetMarkedDeleted())
				{
					// Force resim as follower when pending delete
					DirtyParticleInfo.bResimAsFollower = true;
				}
			}
			GeometryParticle->SetResimType(DirtyParticleInfo.bResimAsFollower ? EResimType::ResimAsFollower : EResimType::FullResim);
		}
	}

	{
		QUICK_SCOPE_CYCLE_COUNTER(ChaosRewindAndResim_RewindParticles);
		for (FDirtyParticleInfo& DirtyParticleInfo : DirtyParticles)
		{
			FGeometryParticleHandle* GeometryParticle = DirtyParticleInfo.GetObjectPtr();
			FKinematicGeometryParticleHandle* KinematicParticle = GeometryParticle->CastToKinematicParticle();
			FPBDRigidParticleHandle* RigidParticle = GeometryParticle->CastToRigidParticle();

			//rewind is about to start, all particles should be in sync at this point
			ensure(GeometryParticle->SyncState() == ESyncState::InSync);
		
			FGeometryParticleStateBase& History = DirtyParticleInfo.GetHistory(); //non-const in case we need to record what's at head for a rewind (CurFrame has already been increased to the next frame)

			const bool bResimAsFollower = DirtyParticleInfo.bResimAsFollower;

			if (GeometryParticle->ResimType() == EResimType::FullResim)
			{
				CachePreResimState(*GeometryParticle);
			}

			if (KinematicParticle)
			{
				// Always clear the kinematic target
				static const FKinematicTarget DefaultKinematicTarget = FKinematicTarget();
				KinematicParticle->SetKinematicTarget(DefaultKinematicTarget);
			}

			bool bAnyChange = false;

			// With bubble resim enabled, we only rewind particles in the bubble		
			bool ShouldRewind = GeometryParticle->ResimType() != EResimType::FrozenDuringResim;
			if (ShouldRewind)
			{
				bAnyChange = RewindHelper(GeometryParticle, bResimAsFollower, History.ParticlePositionRotation, [&](auto Particle, const auto& Data) {Particle->SetXR(Data); });
				bAnyChange |= RewindHelper(KinematicParticle, bResimAsFollower, History.Velocities, [](auto Particle, const auto& Data) { Particle->SetV(Data.V()); Particle->SetW(Data.W()); });
				bAnyChange |= RewindHelper(GeometryParticle, bResimAsFollower, History.NonFrequentData, [Evolution](auto Particle, const auto& Data)
					{
						Evolution->InvalidateParticle(Particle); // Clear collision/constraints before updating NonFrequentData
						Particle->SetNonFrequentData(Data);
					});
				bAnyChange |= RewindHelper(KinematicParticle, bResimAsFollower, History.KinematicTarget, [Evolution](auto Particle, const auto& Data)
					{
						if (Data.GetMode() != EKinematicTargetMode::None)
						{
							Particle->SetKinematicTarget(Data);
							Evolution->GetParticles().MarkMovingKinematic(Particle->Handle());
						}
					});
				bAnyChange |= RewindHelper(RigidParticle, bResimAsFollower, History.Dynamics, [](auto Particle, const auto& Data) {Particle->SetDynamics(Data); });
				bAnyChange |= RewindHelper(RigidParticle, bResimAsFollower, History.DynamicsMisc, [this](auto Particle, const auto& Data) { Solver->SetParticleDynamicMisc(Particle, Data);	});
				bAnyChange |= RewindHelper(RigidParticle, bResimAsFollower, History.MassProps, [](auto Particle, const auto& Data) {Particle->SetMassProps(Data); });
			}

			if (!bResimAsFollower && bAnyChange)
			{
				//particle actually changes not just created/streamed so need to update its state

				//Data changes so send back to GT for interpolation. TODO: improve this in case data ends up being identical in resim
				Evolution->GetParticles().MarkTransientDirtyParticle(DirtyParticleInfo.GetObjectPtr());

				DirtyParticleInfo.DirtyDynamics = INDEX_NONE;	//make sure to undo this as we want to record it again during resim

				//for now just mark anything that changed as enabled during resim. TODO: use bubble
				DirtyParticleInfo.GetObjectPtr()->SetEnabledDuringResim(true);

				// Clear stale friction anchors. Relative shape positions changed during
				// rewind so the cached anchor would generate wrong tangential forces on
				// the first resim step.
				if (RigidParticle)
				{
					RigidParticle->ParticleCollisions().VisitCollisions(
						[](FPBDCollisionConstraint& Constraint) -> ECollisionVisitorResult
						{
							for (FManifoldPoint& MP : Constraint.GetManifoldPoints())
							{
								MP.Flags.bHasStaticFrictionAnchor = false;
							}
							return ECollisionVisitorResult::Continue;
						});
				}
			}

			if (DirtyParticleInfo.InitializedOnStep > RewindFrame)
			{
				//hasn't initialized yet, so disable
				//must do this after rewind because SetDynamicsMisc will re-enable
				//(the disable is a temp way to ignore objects not spawned yet, they weren't really disabled which is why it gets re-enabled)
				Evolution->DisableParticle(DirtyParticleInfo.GetObjectPtr());
			}

			// If we modified the particle, trace it to CVD
			// Bubble Resim: CVD tracing for bubble resim is done at the end for this function, after we freeze particles
#if WITH_CHAOS_VISUAL_DEBUGGER
			if (bAnyChange && !ResimConsoleVars::bIsBubbleResimEnabled)
			{
				ParticlesModifiedDuringRewind.Add(GeometryParticle);
			}
#endif
		}
	}

	// Bubble resim only: "Freeze" particles outside resim bubbles
	FreezeParticlesOutsideResimBubbles();

#if WITH_CHAOS_VISUAL_DEBUGGER
	TraceParticlesModifiedDuringRewind();
#endif

#if !UE_BUILD_SHIPPING
	// For now, just ensure that the joints are InSync
	for(FDirtyJointInfo& DirtyJointInfo : DirtyJoints)
	{
		const FPBDJointConstraintHandle* Joint = DirtyJointInfo.GetObjectPtr();
		//rewind is about to start, all particles should be in sync at this point
		ensure(Joint->SyncState() == ESyncState::InSync);
	}
#endif
}

#if WITH_CHAOS_VISUAL_DEBUGGER
void FRewindData::TraceParticlesModifiedDuringRewind()
{
	if (FChaosVisualDebuggerTrace::IsTracing() && CVDDC_ResimDetails->IsChannelEnabled())
	{
		// With bubble resim, anything that's not static ends up either being frozen (outside the resim bubbles) or marked as FullResim (inside the bubble)
		// So we are better off tracing everything. We're only tracing the ActiveDynamicMovingKinematicParticlesView to try and keep the size down
		// TODO: Implement particle tracing to trace only object state and resim type but on all particles to make sure we've not forgotten some updates
		if (ResimConsoleVars::bIsBubbleResimEnabled)
		{
			if (FPBDRigidsEvolution* Evolution = Solver ? Solver->GetEvolution() : nullptr)
			{
				CVD_TRACE_PARTICLES_VIEW(Evolution->GetParticles().GetActiveDynamicMovingKinematicParticlesView());
			}
		}
		else
		{
			for (const FGeometryParticleHandle* GeometryParticle : ParticlesModifiedDuringRewind)
			{
				CVD_TRACE_PARTICLE(GeometryParticle);
			}
		}
	}
}
#endif

void FRewindData::EndResim()
{
	CVD_SCOPE_TRACE_SOLVER_STEP(CVDDC_ResimDetails, TEXT("End Resim"));
	QUICK_SCOPE_CYCLE_COUNTER(ChaosRewindAndResim_EndResim);

	// Bubble resim only: "Unfreeze" particles that were frozen because they were outside resim bubbles
	UnfreezeParticlesOutsideResimBubbles();

	// Clear the array of resim bubbles
	ResimBubbles.Empty();

#if WITH_CHAOS_VISUAL_DEBUGGER
	ParticlesModifiedDuringRewind.Empty();
#endif

	// Measure the resim time and set a cooldown for that amount of realtime that resim took over the fixed delta time
	const uint64 ResimCycles = FPlatformTime::Cycles64() - ResimStartCycleCounter;
	const uint64 BudgetCycles = Solver ? static_cast<uint64>(Solver->GetAsyncDeltaTime() / FPlatformTime::GetSecondsPerCycle64()) : 0;
	if (ResimCycles > BudgetCycles)
	{
		ResimPerformanceCooldownCycleCounter = FPlatformTime::Cycles64() + (ResimCycles - BudgetCycles);
	}
}

void FRewindData::StepNonResimParticles(const int32 Frame)
{
	QUICK_SCOPE_CYCLE_COUNTER(ChaosRewindAndResim_StepNonResimParticles);
	CVD_SCOPE_TRACE_SOLVER_STEP(CVDDC_ResimDetails, TEXT("Step Non Resim Particles"));

	const FFrameAndPhase FrameAndPhase{ Frame, FFrameAndPhase::PrePushData };
	auto RewindHelper = [FrameAndPhase, this](auto Obj, auto& Property, const auto& RewindFunc) -> bool
	{
		if (auto Val = Property.Read(FrameAndPhase, PropertiesPool))
		{
			return RewindFunc(Obj, *Val);
		}
		return false;
	};

	if (bRewindDataOptimization)
	{
		DynamicHandlesUpdateHolder.Reset(0);
		SleepingHandlesUpdateHolder.Reset(0);
		StaticHandlesUpdateHolder.Reset(0);
		KinematicHandlesUpdateHolder.Reset(0);
	}

	for (FDirtyParticleInfo& DirtyParticleInfo : DirtyParticles)
	{
		FGeometryParticleHandle* GeometryParticle = DirtyParticleInfo.GetObjectPtr();

		if (GeometryParticle->ResimType() != EResimType::ResimAsFollower)
		{
			continue;
		}

		FKinematicGeometryParticleHandle* KinematicParticle = GeometryParticle->CastToKinematicParticle();
		FPBDRigidParticleHandle* RigidParticle = GeometryParticle->CastToRigidParticle();

		bool bHasChanged = false;
		const FGeometryParticleStateBase& History = DirtyParticleInfo.GetHistory();

		if (KinematicParticle)
		{
			// Always clear the kinematic target
			static const FKinematicTarget DefaultKinematicTarget = FKinematicTarget();
			KinematicParticle->SetKinematicTarget(DefaultKinematicTarget);
		}

		//  Set Position and Rotation
		bHasChanged = RewindHelper(GeometryParticle, History.ParticlePositionRotation, [](auto Particle, const auto& Data) -> bool
		{
			if (Particle->GetX() != Data.GetX() || Particle->GetR() != Data.GetR())
			{
				Particle->SetXR(Data);
				return true; 
			}
			return false;
		});

		// Set Velocity and Angular Velocity
		bHasChanged |= RewindHelper(KinematicParticle, History.Velocities, [](auto Particle, const auto& Data) -> bool
		{
			if (Particle->GetV() != Data.GetV() || Particle->GetW() != Data.GetW())
			{
				Particle->SetV(Data.GetV()); Particle->SetW(Data.GetW());
				return true;
			}
			return false;
		});

		// Set kinematic target
		bHasChanged |= RewindHelper(KinematicParticle, History.KinematicTarget, [this](auto Particle, const FKinematicTarget& Data) -> bool
		{
			if (Data.GetMode() != EKinematicTargetMode::None)
			{
				Particle->SetKinematicTarget(Data);
				if (!bRewindDataOptimization)
				{
					Solver->GetEvolution()->GetParticles().MarkMovingKinematic(Particle->Handle()); // Update SOA views with this particle as being moving kinematic
				}
				return true;
			}
			return false;
		});

		if (bRewindDataOptimization)
		{
			// Set disabled true/false and object state
			bHasChanged |= RewindHelper(RigidParticle, History.DynamicsMisc, [this, bHasChanged](auto Particle, const auto& Data) -> bool
				{
					if (Particle == nullptr)
					{
						return false;
					}

					bool bAppliedChange = false;
					if (Particle->Disabled() != Data.Disabled())
					{
						if (Data.Disabled())
						{
							Solver->GetEvolution()->DisableParticle(Particle);
						}
						else
						{
							Solver->GetEvolution()->EnableParticle(Particle);
						}
						bAppliedChange |= true;
					}

					if (Particle->ObjectState() != Data.ObjectState())
					{
						bAppliedChange = true;
					}

					if (bHasChanged || bAppliedChange)
					{
						switch (Data.ObjectState())
						{
							case EObjectStateType::Dynamic:
							{
								// If the particle is dynamic we must fix the collision anchors so that friction doesn't undo the movement
								Particle->ParticleCollisions().VisitCollisions(
									[this, Particle](FPBDCollisionConstraint& Collision)
									{
										Collision.UpdateParticleTransform(Particle);
										return ECollisionVisitorResult::Continue;
									});

								DynamicHandlesUpdateHolder.Add(Particle);
								break;
							}
							case EObjectStateType::Sleeping:
							{
								SleepingHandlesUpdateHolder.Add(Particle);
								break;
							}
							case EObjectStateType::Static:
							{
								StaticHandlesUpdateHolder.Add(Particle);
								break;
							}
							case EObjectStateType::Kinematic:
							{
								KinematicHandlesUpdateHolder.Add(Particle);
								break;
							}
						}
					}

					return bAppliedChange;
				});
		}
		else
		{
			// If XRVW has not changed, or we don't have an active KinematicTarget for the non-resim particle, continue to the next particle
			if (!bHasChanged)
			{
				continue;
			}

			RewindHelper(GeometryParticle, History.NonFrequentData, [this](auto Particle, const auto& Data) -> bool
			{
				Solver->GetEvolution()->InvalidateParticle(Particle); // Clear collision/constraints before updating NonFrequentData
				Particle->SetNonFrequentData(Data);
				return true;
			});
			RewindHelper(RigidParticle, History.Dynamics, [](auto Particle, const auto& Data) -> bool { Particle->SetDynamics(Data); return true; });
			RewindHelper(RigidParticle, History.DynamicsMisc, [this](auto Particle, const auto& Data) -> bool { Solver->SetParticleDynamicMisc(Particle, Data); return true; });
			RewindHelper(RigidParticle, History.MassProps, [](auto Particle, const auto& Data) -> bool { Particle->SetMassProps(Data); return true; });
		}


#if WITH_CHAOS_VISUAL_DEBUGGER
		if (bHasChanged && RigidParticle && FChaosVisualDebuggerTrace::IsTracing() && CVDDC_ResimDetails->IsChannelEnabled())
		{
			CVD_TRACE_PARTICLE(RigidParticle);
		}
#endif
	}

	if (bRewindDataOptimization)
	{
		if (FPBDRigidsEvolution* Evolution = Solver->GetEvolution())
		{
			QUICK_SCOPE_CYCLE_COUNTER(ChaosRewindAndResim_StepNonResimParticles_UpdateObjectsState);

			// Update the particles object states
			Evolution->SetParticlesObjectState(DynamicHandlesUpdateHolder, EObjectStateType::Dynamic, false);
			Evolution->SetParticlesObjectState(SleepingHandlesUpdateHolder, EObjectStateType::Sleeping, false);
			Evolution->SetParticlesObjectState(StaticHandlesUpdateHolder, EObjectStateType::Static, false);
			Evolution->SetParticlesObjectState(KinematicHandlesUpdateHolder, EObjectStateType::Kinematic, false);

			// Update the dirty views
			Evolution->GetParticles().UpdateDirtyViews();
		}
	}
}

template <bool bSkipDynamics, typename TDirtyInfo>
void FRewindData::DesyncIfNecessary(TDirtyInfo& Info, const FFrameAndPhase FrameAndPhase)
{
	ensure(IsResim());	//shouldn't bother with desync unless we're resimming

	auto Handle = Info.GetObjectPtr();
	const auto& History = Info.GetHistory();

	if (Handle->SyncState() == ESyncState::InSync && !History.template IsInSync<bSkipDynamics>(*Handle, FrameAndPhase, PropertiesPool))
	{
		if (!SkipDesyncTest)
		{
			//first time desyncing so need to clear history from this point into the future
			DesyncObject(Info, FrameAndPhase);
		}
	}
}

void FRewindData::FinishFrame()
{
	QUICK_SCOPE_CYCLE_COUNTER(RewindDataFinishFrame);

	if (IsResim())
	{
		FFrameAndPhase FutureFrame{ CurFrame + 1, FFrameAndPhase::PrePushData };

		auto DesyncIfNecessaryHelper = [this, FutureFrame](auto& DirtyObjs)
		{
			for (auto& Info : DirtyObjs)
			{
				if (!Info.bResimAsFollower)
				{
					auto& Handle = *Info.GetObjectPtr();
					if (Handle.ResimType() == EResimType::FullResim)
					{
						//solver doesn't affect dynamics, so no reason to test if they desynced from original sim
						//question: should we skip all other properties? dynamics is a commonly changed one but might be worth skipping everything solver skips
						DesyncIfNecessary</*bSkipDynamics=*/true>(Info, FutureFrame);
					}
				}
			}
		};

		auto FinalResimHelper = [this, FutureFrame](auto& DirtyObjs)
		{
			for (auto& Info : DirtyObjs)
			{
				if (!Info.bResimAsFollower)
				{
					auto& Handle = *Info.GetObjectPtr();
					if (Handle.ResimType() == EResimType::FullResim)
					{
						//Last resim so mark as in sync
						Handle.SetSyncState(ESyncState::InSync);
						Handle.SetEnabledDuringResim(false);

						//Anything saved on upcoming frame (was done during rewind) can be removed since we are now at head
						Info.ClearPhaseAndFuture(FutureFrame);
					}
				}
				// Trying if this gets rid of the ensure on subsequent resims where some particles were left as hard desync, I don't know why
				// TODO: This needs urgent attention (retest, does it still happen?) because it may indicate something is not done correctly in bubble resim
				if (ResimConsoleVars::bIsBubbleResimEnabled)
				{
					auto& Handle = *Info.GetObjectPtr();
					Handle.SetSyncState(ESyncState::InSync);
				}
			}
		};

		bool bIsFinalResim = IsFinalResim();

		if (!bIsFinalResim)
		{
			if (!bRewindDataOptimization)
			{
				DesyncIfNecessaryHelper(DirtyParticles);
				DesyncIfNecessaryHelper(DirtyJoints);
			}
		}
		else
		{
			FinalResimHelper(DirtyParticles);
			FinalResimHelper(DirtyJoints);
		}
	}

	++CurFrame;
	LatestFrame = FMath::Max(LatestFrame, CurFrame);
}

void FRewindData::SkipFrames(int32 NumFramesToSkip)
{
	// Advance frame counters only - FramesSaved is intentionally not incremented because
	// these frames were already saved during the original simulation.
	CurFrame += NumFramesToSkip;
	LatestFrame = FMath::Max(LatestFrame, CurFrame);
}

void FRewindData::DumpHistory_Internal(const int32 FramePrintOffset, const FString& Filename)
{
	FStringOutputDevice Out;
	const int32 EarliestFrame = LatestFrame - FramesSaved;
	for(int32 Frame = EarliestFrame; Frame < CurFrame; ++Frame)
	{
		for (int32 Phase = 0; Phase < FFrameAndPhase::EParticleHistoryPhase::NumPhases; ++Phase)
		{
			for(const FDirtyParticleInfo& Info : DirtyParticles)
			{
				Out.Logf(TEXT("Frame:%d Phase:%d\n"), Frame + FramePrintOffset, Phase);
				FGeometryParticleState State = GetPastStateAtFrame(*Info.GetObjectPtr(), Frame, (FFrameAndPhase::EParticleHistoryPhase)Phase);
				Out.Logf(TEXT("%s\n"), *State.ToString());
			}

			for (const FDirtyJointInfo& Info : DirtyJoints)
			{
				Out.Logf(TEXT("Frame:%d Phase:%d\n"), Frame + FramePrintOffset, Phase);

				FJointState State = GetPastJointStateAtFrame(*Info.GetObjectPtr(), Frame, (FFrameAndPhase::EParticleHistoryPhase)Phase);
				Out.Logf(TEXT("%s\n"), *State.ToString()); 
			}
		}
	}

	FString Path = FPaths::ProfilingDir() + FString::Printf(TEXT("/RewindData/%s_%d_%d.txt"), *Filename, EarliestFrame + FramePrintOffset, CurFrame - 1 + FramePrintOffset);
	FFileHelper::SaveStringToFile(Out, *Path);
	UE_LOGF(LogChaos, Warning, "Saved:%ls", *Path);
}

bool FRewindData::GetUseCollisionResimCache() const
{
	return Solver ? Solver->GetUseCollisionResimCache() : false;
}

int32 ResizeHistoryFromTickRate = 1;
FAutoConsoleVariableRef CVarResimResizeHistoryFromTickRate(TEXT("p.Resim.ResizeHistoryFromTickRate"), ResizeHistoryFromTickRate, TEXT("When set to 1, RewindData will resize the history if needed when physics tick-rate changes."));

void FRewindData::ResizeRewindDataIfNeeded()
{
	if (!AllowResizingFromTickTrate || ResizeHistoryFromTickRate != 1)
	{
		return;
	}

	// Check if tick-rate has changed since RewindData was initialized and if so, reinitialize
	if (Solver && !IsResim() && FMath::IsNearlyEqual(InitializedDeltaTime, Solver->GetAsyncDeltaTime()) == false)
	{
		Solver->EnableRewindCapture();
		BlockResim();
		if (RewindDataResize.IsBound())
		{
			RewindDataResize.Broadcast(Managers.Capacity());
		}
	}
}

CHAOS_API int32 SkipDesyncTest = 0;
FAutoConsoleVariableRef CVarSkipDesyncTest(TEXT("p.SkipDesyncTest"), SkipDesyncTest, TEXT("Skips hard desync test, this means all particles will assume to be clean except spawning at different times. This is useful for a perf lower bound, not actually correct"));

void FRewindData::AdvanceFrameImp(IResimCacheBase* ResimCache)
{
	ResizeRewindDataIfNeeded();

	FramesSaved = FMath::Min(FramesSaved + 1, static_cast<int32>(Managers.Capacity()));

	const bool bHasResimCache = ResimCache != nullptr;
	const int32 EarliestFrame = CurFrame - FramesSaved;
	const FFrameAndPhase FrameAndPhase{ CurFrame, FFrameAndPhase::PostCallbacks };

	auto AdvanceHelper = [this, EarliestFrame, FrameAndPhase](auto& DirtyObjects, const auto& DesyncFunc, const auto& AdvanceDirtyFunc)
	{
		const int32 InitialNumDirtyObjects = DirtyObjects.Num();
		for (int32 DirtyIdx = InitialNumDirtyObjects - 1; DirtyIdx >= 0; --DirtyIdx)
		{
			auto& Info = DirtyObjects.GetDenseAt(DirtyIdx);

			ensure(IsResimAndInSync(*Info.GetObjectPtr()) || Info.GetHistory().IsClean(FrameAndPhase));  //Sim hasn't run yet so PostCallbacks (sim results) should be clean
	
			//if hasn't changed in a while stop tracking
			if (Info.LastDirtyFrame < EarliestFrame && !IsResim())
			{
				RemoveObject(Info.GetObjectPtr(), EAllowShrinking::No);
			}
			else
			{
				auto Handle = Info.GetObjectPtr();
				Info.bResimAsFollower = Handle->ResimType() == EResimType::ResimAsFollower;

				if (IsResim())
				{
					if (!bRewindDataOptimization && Handle->ResimType() == EResimType::FullResim)
					{
						DesyncIfNecessary</*bSkipDynamics=*/false>(Info, FrameAndPhase);
					}

					if (Handle->SyncState() != ESyncState::InSync && !SkipDesyncTest)
					{
						Handle->SetEnabledDuringResim(true);	//for now just mark anything out of sync as resim enabled. TODO: use bubble
						DesyncFunc(Handle);
					}

					Info.bNeedsResim = false;
				}

				AdvanceDirtyFunc(Info, Handle);
			}
		}
		if (InitialNumDirtyObjects > 0)
		{
			DirtyObjects.Shrink();
		}
	};

	TArray<FGeometryParticleHandle*> DesyncedParticles;
	if (IsResim() && bHasResimCache)
	{
		DesyncedParticles.Reserve(DirtyParticles.Num());
	}

	AdvanceHelper(DirtyParticles,
		[&DesyncedParticles, bHasResimCache](FGeometryParticleHandle* DesyncedHandle)
		{
			if (bHasResimCache)
			{
				DesyncedParticles.Add(DesyncedHandle);
			}
		},

		[this, FrameAndPhase](FDirtyParticleInfo& Info, FGeometryParticleHandle* Handle)
		{
			if (!bRewindDataOptimization && Info.DirtyDynamics == CurFrame && !IsResimAndInSync(*Handle))
			{
				//we only need to check the cast because right now there's no property system on PT, so any time a sim callback touches a particle we just mark it as dirty dynamics
				if (auto Rigid = Handle->CastToRigidParticle())
				{
					//sim callback is finished so record the dynamics before solve starts
					Info.MarkDirty(CurFrame);
					FGeometryParticleStateBase& Latest = Info.GetHistory();
					Latest.Dynamics.WriteAccessMonotonic(FrameAndPhase, PropertiesPool).CopyFrom(*Rigid);
				}
			}
		});

	AdvanceHelper(DirtyJoints, [](const FPBDJointConstraintHandle*) {}, [](const FDirtyJointInfo&, const FPBDJointConstraintHandle*) {});

	//TODO: if joint is desynced we should desync particles as well
	//If particle of joint is desynced, we need to make sure the joint is reconsidered too for optimization, though maybe not "desynced"

	if (IsResim() && bHasResimCache)
	{
		ResimCache->SetDesyncedParticles(MoveTemp(DesyncedParticles));
	}
}

#ifndef REWIND_DESYNC
#define REWIND_DESYNC 0
#endif

void FRewindData::PushGTDirtyData(const FDirtyPropertiesManager& SrcManager,const int32 SrcDataIdx,const FDirtyProxy& Dirty, const FShapeDirtyData* ShapeDirtyData)
{
	//This records changes enqueued by GT.
	bNeedsSave = true;

	IPhysicsProxyBase* Proxy = Dirty.Proxy;
	if (Proxy == nullptr)
	{
		return;
	}

	//Helper to group most of the common logic about push data recording
	//NOTE: when possible use passed in CopyFunc to do work, if lambda returns false you cannot record to history buffer
	auto CopyHelper = [this, Proxy](auto Object, const auto& CopyFunc) -> bool
	{
		//Don't bother tracking static particles. We assume they stream in and out and don't need to be rewound
		//TODO: find a way to skip statics that stream in and out - gameplay can technically spawn/destroy these so we can't just ignore statics
		/*if(PTParticle->CastToKinematicParticle() == nullptr)
		{
			return;
		}*/

		//During a resim the same exact push data comes from gt
		//If the particle is already in sync, it will stay in sync so no need to touch history
		if (IsResim() && Object->SyncState() == ESyncState::InSync)
		{
			return false;
		}

		if (IsResim() && Proxy->GetInitializedStep() == CurFrame)
		{
			//Particle is reinitialized, since it's out of sync it must be at a different time
			//So make sure it's considered during resim
			//TODO: should check if in bubble
			Object->SetEnabledDuringResim(true);
		}

		auto& Info = FindOrAddDirtyObj(*Object, Proxy->IsInitialized() ? INDEX_NONE : CurFrame);
		Info.MarkDirty(CurFrame);
		auto& Latest = Info.GetHistory();

		//At this point all phases should be clean
		ensure(Latest.IsClean(FFrameAndPhase{ CurFrame, FFrameAndPhase::PrePushData }));

		//Most objects never change but may be created/destroyed often due to streaming
		//To avoid useless writes we call this function before PushData is processed.
		//This means we will skip objects that are streamed in since they never change
		//So if Proxy has initialized it means the particle isn't just streaming in, it's actually changing
		if (Info.InitializedOnStep < CurFrame)
		{
			CopyFunc(Latest);
		}

		ensure(Latest.IsClean(FFrameAndPhase{ CurFrame, FFrameAndPhase::PostPushData }));   //PostPushData is untouched
		ensure(Latest.IsClean(FFrameAndPhase{ CurFrame, FFrameAndPhase::PostCallbacks }));	//PostCallback is untouched

		return true;
	};

	auto DirtyPropHelper = [this, &Dirty](auto& Property, const EChaosPropertyFlags PropName, const auto& Object)
	{
		if (Dirty.PropertyData.IsDirty(PropName))
		{
			auto& Data = Property.WriteAccessMonotonic(FFrameAndPhase{ CurFrame, FFrameAndPhase::PrePushData }, PropertiesPool);
			CopyDataFromObject(Data, Object);
		}
	};

	switch(Proxy->GetType())
	{
	case EPhysicsProxyType::SingleParticleProxy:
	{
		FSingleParticlePhysicsProxy* ParticleProxy = static_cast<FSingleParticlePhysicsProxy*>(Proxy);
		if (ParticleProxy == nullptr)
		{
			break;
		}

		FGeometryParticleHandle* PTParticle = ParticleProxy->GetHandle_LowLevel();
		if (PTParticle == nullptr)
		{
			break;
		}

		if (bRewindDataOptimization)
		{
			/** When using optimization, if this is the first time the particle is dirtied, cache the particles current state and register as dirty */
			if (FDirtyParticleInfo* DirtyParticleInfo = FindDirtyObj(*PTParticle))
			{
				DirtyParticleInfo->MarkDirty(CurFrame);
			}
			else // Not yet tracked as dirty
			{
				// Call FindOrAddDirtyObj here to set the InitializedOnFrame, if the proxy is not yet initialized
				FindOrAddDirtyObj(*PTParticle, ParticleProxy->IsInitialized() ? INDEX_NONE : CurFrame);
				CacheDirtyParticleData(PTParticle, FFrameAndPhase::PrePushData, /*bDirty*/true);				
			}
			break;
		}

		const bool bKeepRecording = CopyHelper(PTParticle, [PTParticle, &DirtyPropHelper](FGeometryParticleStateBase& Latest)
		{
			DirtyPropHelper(Latest.ParticlePositionRotation, EChaosPropertyFlags::XR, *PTParticle);
			DirtyPropHelper(Latest.NonFrequentData, EChaosPropertyFlags::NonFrequentData, *PTParticle);

			if (auto Kinematic = PTParticle->CastToKinematicParticle())
			{
				DirtyPropHelper(Latest.Velocities, EChaosPropertyFlags::Velocities, *Kinematic);
				DirtyPropHelper(Latest.KinematicTarget, EChaosPropertyFlags::KinematicTarget, *Kinematic);

				if (auto Rigid = Kinematic->CastToRigidParticle())
				{
					DirtyPropHelper(Latest.DynamicsMisc, EChaosPropertyFlags::DynamicMisc, *Rigid);
					DirtyPropHelper(Latest.MassProps, EChaosPropertyFlags::MassProps, *Rigid);
				}
			}
		});

		if (bKeepRecording)
		{
			//Dynamics are not available at head (sim zeroes them out), so we have to record them as PostPushData (since they're applied as part of PushData)
			if (auto NewData = Dirty.PropertyData.FindDynamics(SrcManager, SrcDataIdx))
			{
				FDirtyParticleInfo& Info = FindOrAddDirtyObj(*PTParticle, ParticleProxy->IsInitialized() ? INDEX_NONE : CurFrame);
				Info.MarkDirty(CurFrame);
				FGeometryParticleStateBase& Latest = Info.GetHistory();
				const FFrameAndPhase PostPushData{ CurFrame, FFrameAndPhase::PostPushData };
				Latest.Dynamics.WriteAccessMonotonic(PostPushData, PropertiesPool) = *NewData;
				Info.DirtyDynamics = CurFrame;	//Need to save the dirty dynamics into the next phase as well (it's possible a callback will stomp the dynamics value, so that's why it's pending)

				ensure(Latest.IsCleanExcludingDynamics(PostPushData)); //PostPushData is untouched except for dynamics
			}
		}
		break;
	}
	case EPhysicsProxyType::JointConstraintType:
	{
		FJointConstraintPhysicsProxy* JointProxy = static_cast<FJointConstraintPhysicsProxy*>(Proxy);
		if (JointProxy == nullptr)
		{
			break;
		}
		FPBDJointConstraintHandle* Joint = JointProxy->GetHandle();
		if (Joint == nullptr)
		{
			break;
		}

		if (bRewindDataOptimization)
		{
			/** When using optimization, if this is the first time the joint is dirtied, cache the joints current state and register as dirty */
			if (FDirtyJointInfo* DirtyJointInfo = FindDirtyObj(*Joint))
			{
				DirtyJointInfo->MarkDirty(CurFrame);
			}
			else // Not yet tracked as dirty
			{
				// Call FindOrAddDirtyObj here to set the InitializedOnFrame, if the proxy is not yet initialized
				FindOrAddDirtyObj(*Joint, JointProxy->IsInitialized() ? INDEX_NONE : CurFrame);
				CacheDirtyJointData(Joint, FFrameAndPhase::PrePushData, /*bDirty*/true);
			}
			break;
		}

		CopyHelper(Joint, [Joint, &DirtyPropHelper](FJointStateBase& Latest)
		{
			DirtyPropHelper(Latest.JointSettings, EChaosPropertyFlags::JointSettings, *Joint);
		});
		break;
	}
	default:
	{
		ensure(false);	//Unsupported proxy type
	}
	}
}

void FRewindData::SpawnProxyIfNeeded(FSingleParticlePhysicsProxy& Proxy)
{
	if(Proxy.GetInitializedStep() > CurFrame)
	{
		FGeometryParticleHandle* Handle = Proxy.GetHandle_LowLevel();
		FDirtyParticleInfo& Info = FindOrAddDirtyObj(*Handle, CurFrame);

		Solver->GetEvolution()->EnableParticle(Handle);
		if(Proxy.GetInitializedStep() != CurFrame)
		{
			DesyncObject(Info, FFrameAndPhase{ Proxy.GetInitializedStep(), FFrameAndPhase::PrePushData });	//Spawned earlier so mark as desynced from that first frame
			Proxy.SetInitialized(CurFrame);
			Info.InitializedOnStep = CurFrame;
		}
	}
}

void FRewindData::DisableInHistoryFromFrame(IPhysicsProxyBase* PhysicsProxy, int32 DisableFromFrame)
{
	if (!Solver || !Solver->GetEvolution())
	{
		return;
	}
	
	// If no from frame is specified, go back the amount of frames we are currently forward predicted
	if (DisableFromFrame == INDEX_NONE)
	{
		DisableFromFrame = LatestFrame - FMath::RoundToInt32(NumPredictedFramesAverage);
	}

	auto DisableInHistoryHelper = [&, DisableFromFrame](const FGeometryParticleHandle* Handle)
	{
		if (FDirtyParticleInfo* Info = FindDirtyObj(*Handle))
		{
			FGeometryParticleStateBase& History = Info->GetHistory();
			FFrameAndPhase FrameAndPhase { DisableFromFrame , FFrameAndPhase::PrePushData };
			for (int32 Frame = DisableFromFrame; Frame <= LatestFrame; Frame++)
			{
				FrameAndPhase.Frame = Frame;

				// Check if we have particle data in history to overwrite
				if (!History.DynamicsMisc.IsClean(FrameAndPhase))
				{
					FParticleDynamicMisc& DynamicsMisc = History.DynamicsMisc.Insert(FrameAndPhase, PropertiesPool);
					DynamicsMisc.SetDisabled(true);
				}

				// Check if we have target state data in history to overwrite
				if (!History.TargetStates.IsClean(FrameAndPhase))
				{
					FParticleDynamicMisc& DynamicsMisc = History.TargetStates.Insert(FrameAndPhase, PropertiesPool);
					DynamicsMisc.SetDisabled(true);
				}
			}
		}
	};

	if (PhysicsProxy)
	{
		switch (PhysicsProxy->GetType())
		{
		case EPhysicsProxyType::SingleParticleProxy:
		{
			if (const FSingleParticlePhysicsProxy* Proxy = static_cast<const FSingleParticlePhysicsProxy*>(PhysicsProxy))
			{
				if (const FGeometryParticleHandle* Handle = Proxy->GetHandle_LowLevel())
				{
					DisableInHistoryHelper(Handle);
				}
			}
			break;
		}
		case EPhysicsProxyType::ClusterUnionProxy:
		{
			if (const FClusterUnionPhysicsProxy* Proxy = static_cast<const FClusterUnionPhysicsProxy*>(PhysicsProxy))
			{
				if (const FGeometryParticleHandle* Handle = Proxy->GetParticle_Internal())
				{
					DisableInHistoryHelper(Handle);

					// Iterate over and process all owned particles in the union also
					FClusterUnionManager& ClusterUnionManager =	Solver->GetEvolution()->GetRigidClustering().GetClusterUnionManager();
					if (const Chaos::FClusterUnion* Union = ClusterUnionManager.FindClusterUnionFromParticle(Handle->CastToRigidParticle()))
					{
						for (Chaos::FPBDRigidParticleHandle* Child : Union->GeometryChildParticles)
						{
							if (Child)
							{
								DisableInHistoryHelper(Child);
							}
						}
					}
				}
			}
			break;
		}
		case EPhysicsProxyType::GeometryCollectionType:
		{
			if (const FGeometryCollectionPhysicsProxy* Proxy = static_cast<const FGeometryCollectionPhysicsProxy*>(PhysicsProxy))
			{
				for (const FGeometryParticleHandle* Handle : Proxy->GetUnorderedParticles_Internal())
				{
					if (Handle)
					{
						DisableInHistoryHelper(Handle);
					}
				}
			}
			break;
		}
		}
	}
}

void FRewindData::CachePreResimState(FGeometryParticleHandle& Handle)
{
	if (!Solver || !Solver->GetEvolution())
	{
		return;
	}

	const IPhysicsProxyBase* PhysicsProxy = Handle.PhysicsProxy();
	if (!PhysicsProxy || PhysicsProxy->GetMarkedDeleted())
	{
		return;
	}

	// Find or add pre-resim error for the particle that has an error
	auto FindOrAddPreResimState = [this](FGeometryParticleHandle* ParticleHandle) -> FParticleSimpleState&
		{
			if (FParticleSimpleState* Found = DirtyParticlePreResimState.Find(ParticleHandle))
			{
				return *Found;
			}

			return DirtyParticlePreResimState.Add(ParticleHandle, FParticleSimpleState(*ParticleHandle));
		};

	auto CacheState = [&](FGeometryParticleHandle* ParticleHandle)
		{
			if (!ParticleHandle)
			{
				return;
			}

			if (const FPBDRigidParticleHandle* RigidParticle = ParticleHandle->CastToRigidParticle())
			{
				if (RigidParticle->Disabled())
				{
					return;
				}
			}

			FParticleSimpleState& PreResimState = FindOrAddPreResimState(ParticleHandle);

			// Cache dirty particle XR before a resimulation
			PreResimState.CacheXR(ParticleHandle->GetX(), ParticleHandle->GetR());

			/** Cache dirty kinematic or rigid particle VW before a resimulation
			* NOTE: Rigids are based on kinematic so can always cast to kinematic */
			if (FKinematicGeometryParticleHandle* KinematicHandle = ParticleHandle->CastToKinematicParticle())
			{
				PreResimState.CacheVW(KinematicHandle->GetV(), KinematicHandle->GetW());
			}
		};

	// Cache dirty particle state before a resimulation
	CacheState(&Handle);

	// If particle is a Cluster Union, also cache child particles
	if (PhysicsProxy->GetType() == EPhysicsProxyType::ClusterUnionProxy)
	{
		FClusterUnionManager& ClusterUnionManager = Solver->GetEvolution()->GetRigidClustering().GetClusterUnionManager();
		if (const Chaos::FClusterUnion* Union = ClusterUnionManager.FindClusterUnionFromParticle(Handle.CastToRigidParticle()))
		{
			for (Chaos::FPBDRigidParticleHandle* Child : Union->GeometryChildParticles)
			{
				if (Child)
				{
					CacheState(Child);
				}
			}
		}
	}
}

void FRewindData::GetPreResimState(Chaos::FGeometryParticleHandle* GeometryParticle, FParticleSimpleState*& OutPreResimState)
{
	OutPreResimState = DirtyParticlePreResimState.Find(GeometryParticle);
}

void FRewindData::RemovePreResimState(Chaos::FGeometryParticleHandle* GeometryParticle)
{
	DirtyParticlePreResimState.Remove(GeometryParticle, EAllowShrinking::No);
}

template<>
void FRewindData::AccumulateErrorIfNecessary(FGeometryParticleHandle& Obj, const FFrameAndPhase FrameAndPhase)
{
	FParticleSimpleState* PreResimState = DirtyParticlePreResimState.Find(&Obj);
	if (!PreResimState)
	{
		return;
	}

	// Get the error offset after a correction
	const FVec3 ErrorX = PreResimState->GetX() - Obj.GetX();
	FQuat ErrorR = Obj.GetR().Inverse() * PreResimState->GetR(); // ErrorR in local space 
	ErrorR.EnforceShortestArcWith(FQuat::Identity);
	ErrorR.Normalize();

	// Check if error is large enough to hide behind render interpolation
	if (!ErrorX.IsNearlyZero(0.1) || !ErrorR.IsIdentity(0.02))
	{
		// Find or add FDirtyParticleErrorInfo for the particle that has an error
		FDirtyParticleErrorInfo& ErrorInfo = [&]() ->FDirtyParticleErrorInfo&
			{
				if (FDirtyParticleErrorInfo* Found = DirtyParticleErrors.Find(&Obj))
				{
					return *Found;
				}

				return DirtyParticleErrors.Add(&Obj, FDirtyParticleErrorInfo(Obj));
			}();

		// Cache error for particle 
		ErrorInfo.AccumulateError(ErrorX, ErrorR);
	}
}

// Move post-resim error correction data from RewindData to FPullPhysicsData for marshaling to GT where it can be used in render interpolation
void FRewindData::BufferPhysicsResults(TMap<const IPhysicsProxyBase*, struct FDirtyRigidParticleReplicationErrorData>& DirtyRigidErrors)
{
	if (IsFinalResim())
	{
		FFrameAndPhase FutureFrame{ CurFrame + 1, FFrameAndPhase::PrePushData };

		auto ErrorDataHelper = [this, FutureFrame](auto& DirtyObjs)
			{
				for (auto& Info : DirtyObjs)
				{
					if (!Info.GetObjectPtr() || Info.GetObjectPtr()->ResimType() != EResimType::FullResim)
					{
						continue;
					}

					// Cache the correction offset after a resimulation
					AccumulateErrorIfNecessary(*Info.GetObjectPtr(), FutureFrame);
				}
			};

		ErrorDataHelper(DirtyParticlePreResimState);
		DirtyParticlePreResimState.Reset();
	}

	DirtyRigidErrors.Reserve(DirtyParticleErrors.Num());

	for (const FDirtyParticleErrorInfo& ErrorInfo : DirtyParticleErrors)
	{
		FDirtyRigidParticleReplicationErrorData ErrorData;
		ErrorData.ErrorX = ErrorInfo.GetErrorX();
		ErrorData.ErrorR = ErrorInfo.GetErrorR();

		if (const IPhysicsProxyBase* PhysicsProxy = static_cast<const IPhysicsProxyBase*>(ErrorInfo.GetObjectPtr()->PhysicsProxy()))
		{
			DirtyRigidErrors.Add(PhysicsProxy, ErrorData);
		}
	}

	DirtyParticleErrors.Reset();
}

void FRewindData::MarkDirtyFromPT(FGeometryParticleHandle& Handle)
{
	if (bRewindDataOptimization)
	{
		/** When using optimization, if this is the first time the joint is dirtied, cache the joints current state and register as dirty */
		if (FDirtyParticleInfo* DirtyParticleInfo = FindDirtyObj(Handle))
		{
			DirtyParticleInfo->MarkDirty(CurFrame);
		}
		else // Not yet tracked as dirty
		{
			CacheDirtyParticleData(&Handle, FFrameAndPhase::PrePushData, /*bDirty*/true);
		}
		return;
	}

	FDirtyParticleInfo& Info = FindOrAddDirtyObj(Handle);
	Info.DirtyDynamics = CurFrame;

	const bool bRecordingHistory = !IsResimAndInSync(Handle);

	Info.MarkDirty(CurFrame);
	FGeometryParticleStateBase& Latest = Info.GetHistory();

	//TODO: use property system
	//For now we just dirty all PT properties that we typically use
	//This means sim callback can't modify mass, geometry, etc... (only properties touched by this function)
	//Note these same properties are sent back to GT, so it's not just this function that needs updating

	const FFrameAndPhase FrameAndPhase{ CurFrame, FFrameAndPhase::PostPushData };

	if (bRecordingHistory || Latest.ParticlePositionRotation.IsClean(FrameAndPhase))
	{
		if (auto Data = Latest.ParticlePositionRotation.WriteAccessNonDecreasing(FrameAndPhase, PropertiesPool))
		{
			Data->CopyFrom(Handle);
		}
	}

	if (auto Kinematic = Handle.CastToKinematicParticle())
	{
		if (bRecordingHistory || Latest.Velocities.IsClean(FrameAndPhase))
		{
			if (auto Data = Latest.Velocities.WriteAccessNonDecreasing(FrameAndPhase, PropertiesPool))
			{
				Data->CopyFrom(*Kinematic);
			}
		}

		if (auto Rigid = Kinematic->CastToRigidParticle())
		{
			if (bRecordingHistory || Latest.DynamicsMisc.IsClean(FrameAndPhase))
			{
				if (auto Data = Latest.DynamicsMisc.WriteAccessNonDecreasing(FrameAndPhase, PropertiesPool))
				{
					Data->CopyFrom(*Rigid);
				}
			}
		}
	}
}

void FRewindData::MarkDirtyJointFromPT(FPBDJointConstraintHandle& Handle)
{
	if (bRewindDataOptimization)
	{
		/** When using optimization, if this is the first time the joint is dirtied, cache the joints current state and register as dirty */
		if (FDirtyJointInfo* DirtyJointInfo = FindDirtyObj(Handle))
		{
			DirtyJointInfo->MarkDirty(CurFrame);
		}
		else // Not yet tracked as dirty
		{
			CacheDirtyJointData(&Handle, FFrameAndPhase::PrePushData, /*bDirty*/true);
		}
		return;
	}

	const bool bRecordingHistory = !IsResimAndInSync(Handle);

	FDirtyJointInfo& Info = FindOrAddDirtyObj(Handle);
	Info.MarkDirty(CurFrame);
	FJointStateBase& Latest = Info.GetHistory();

	//TODO: use property system

	const FFrameAndPhase FrameAndPhase{ CurFrame, FFrameAndPhase::PostPushData };

	if (bRecordingHistory || Latest.JointSettings.IsClean(FrameAndPhase))
	{
		if (auto Data = Latest.JointSettings.WriteAccessNonDecreasing(FrameAndPhase, PropertiesPool))
		{
			CopyDataFromObject(*Data, Handle);
		}
	}
}

void FRewindData::ClearPhaseAndFuture(FGeometryParticleHandle& Handle, int32 Frame, FFrameAndPhase::EParticleHistoryPhase Phase)
{
	FDirtyParticleInfo& Info = FindOrAddDirtyObj(Handle);
	const FFrameAndPhase FrameAndPhase{ Frame, Phase };
	Info.ClearPhaseAndFuture(FrameAndPhase);
}

void FRewindData::ExtendHistoryWithFrame(const int32 Frame)
{
	FramesSaved = FMath::Max(CurFrame - Frame+1, FramesSaved);
}

// todo, implement into settings
enum class EResimFrameValidation : int32
{
	FullValidation = 0, // No leniency, validate all dirty particle
	IslandValidation = 1, // Validate dirty particles inside the islands that have resim trigger particles in them
	TriggerParticleValidation = 2 // Only validate the resim triggering particle(s)
};
CHAOS_API int32 ResimFrameValidation = static_cast<int32>(EResimFrameValidation::IslandValidation);
FAutoConsoleVariableRef CVarResimFrameValidationLeniency(TEXT("p.Resim.ResimFrameValidation"), ResimFrameValidation, TEXT("0 = no leniency, all dirty particles need a valid target. 1 = Island leniency, all particles in resim islands need a valid target. 2 = Full leniency, only the particle triggering the resim need a valid target."));
CHAOS_API int32 FindValidFrameNFramesBehind = 2;
FAutoConsoleVariableRef CVarResimFindValidFrameNFramesBehind(TEXT("p.Resim.FindValidFrameNFramesBehind"), FindValidFrameNFramesBehind, TEXT("Number of frames to allow searching for a valid frame behind the requested frame."));
CHAOS_API float FindValidFrameNPercentBehind = 0.1f;
FAutoConsoleVariableRef CVarResimFindValidFrameNPercentBehind(TEXT("p.Resim.FindValidFrameNPercentBehind"), FindValidFrameNPercentBehind, TEXT("Value between 0.0 -> 1.0 as percent of rewind history size to allow searching for a valid frame behind the requested frame. Example: If rewind history is 60 frames long and we allow looking 10% frames behind, we will allow to look 6 frames back from the requested frame."));
CHAOS_API bool bResimIncompleteHistory = true;
FAutoConsoleVariableRef CVarResimIncompleteHistory(TEXT("p.Resim.IncompleteHistory"), bResimIncompleteHistory, TEXT("If a valid resim frame can't be found, use the requested resim frame and perform a resimulation with incomplete data."));
CHAOS_API bool bFindValidInputHistory = true;
FAutoConsoleVariableRef CVarResimFindValidInputHistory(TEXT("p.Resim.FindValidInputHistory"), bFindValidInputHistory, TEXT("If the particle that needs resimulation has custom input history, find a valid resim frame where inputs are available."));
CHAOS_API bool bFindValidStateHistory = true;
FAutoConsoleVariableRef CVarResimFindValidStateHistory(TEXT("p.Resim.FindValidStateHistory"), bFindValidStateHistory, TEXT("If the particle that needs resimulation has custom state history, find a valid resim frame where states are available."));
CHAOS_API bool bUseParticleResimAsFollowerDuringTargetValidation = false;
FAutoConsoleVariableRef CVarUseParticleResimAsFollowerDuringTargetValidation(TEXT("p.Resim.UseParticleResimAsFollowerDuringTargetValidation"), bUseParticleResimAsFollowerDuringTargetValidation, TEXT("If disabled, do not use the particle's ResimAsFollower flag when trying to find a valid resim frame."));

int32 FRewindData::FindValidResimFrame(const int32 RequestedFrame)
{
	int32 ValidFrame = INDEX_NONE;
	int32 ValidTargetFrame = INDEX_NONE;

	if (RequestedFrame <= BlockResimFrame)
	{
		if (Chaos::ResimConsoleVars::bEnableRewindLogs)
		{
			UE_LOGF(LogChaos, Log, "CLIENT | PT | FindValidResimFrame | Resim is blocked | BlockResimFrame: %d | RequestedFrame: %d", BlockResimFrame, RequestedFrame);
		}
		return ValidFrame;
	}

	EnsureIsInPhysicsThreadContext();

	auto TargetFinderHelper = [&](FDirtyParticleInfo* DirtyParticleInfo, const FFrameAndPhase FrameAndPhase) -> bool
	{
		bool bValid = true;
		bool bResimAsFollower = false;
		if (bUseParticleResimAsFollowerDuringTargetValidation)
		{
			bResimAsFollower = DirtyParticleInfo->bResimAsFollower;
		}
		FGeometryParticleStateBase& History = DirtyParticleInfo->GetHistory();
		if (const FParticleDynamicMisc* DynamicMisc = History.DynamicsMisc.Read(FrameAndPhase, PropertiesPool))
		{
			if (!DynamicMisc->Disabled() && (DynamicMisc->ObjectState() == EObjectStateType::Dynamic) && !History.TargetPositions.IsEmpty() && !History.TargetVelocities.IsEmpty() && !History.TargetStates.IsEmpty())
			{
				if (bResimAsFollower || History.TargetPositions.IsClean(FrameAndPhase) || History.TargetVelocities.IsClean(FrameAndPhase) || History.TargetStates.IsClean(FrameAndPhase))
				{
					bValid = false;
				}
			}
		}

		return bValid;
	};

	auto CustomDataFinderHelper = [&](FDirtyParticleInfo* DirtyParticleInfo, const FFrameAndPhase FrameAndPhase) -> bool
	{
		if (!DirtyParticleInfo->bNeedsResim)
		{
			return true;
		}

		bool bValid = true;
		FGeometryParticleHandle* Handle = DirtyParticleInfo->GetObjectPtr();
		
		if (bFindValidInputHistory)
		{
			if (const TWeakPtr<FBaseRewindHistory>* InputHistory = InputParticleHistories.Find(Handle))
			{
				if (InputHistory->IsValid() && !InputHistory->Pin().Get()->HasValidData(FrameAndPhase.Frame))
				{
					bValid = false;
				}
			}
		}

		if (bValid && bFindValidStateHistory)
		{
			if (const TWeakPtr<FBaseRewindHistory>* StateHistory = StateParticleHistories.Find(Handle))
			{
				if (StateHistory->IsValid() && !StateHistory->Pin().Get()->HasValidData(FrameAndPhase.Frame))
				{
					bValid = false;
				}
			}
		}

		return bValid;
	};

	Private::FPBDIslandManager& IslandManager = Solver->GetEvolution()->GetIslandManager();

	// Cache all particles in islands that have a resim triggering particle
	if (ResimFrameValidation == static_cast<int32>(EResimFrameValidation::IslandValidation))
	{
		IslandValidationIslands.Reset();
		IslandValidationIslandParticles.Reset();
		for (FDirtyParticleInfo& DirtyParticleInfo : DirtyParticles)
		{
			FGeometryParticleHandle* Handle = DirtyParticleInfo.GetObjectPtr();
			if (IslandManager.GetParticleResimFrame(Handle) != INDEX_NONE)
			{
				IslandManager.FindParticleIslands(Handle, OUT IslandValidationIslands);
			}

			/** If the particle needs resim, add it to the IslandValidationIslandParticles array to ensure it will be processed
			* particles that are not in contact with anything doesn't have an island and would be missed */
			if (DirtyParticleInfo.bNeedsResim)
			{
				IslandValidationIslandParticles.AddUnique(Handle);
			}
		}

		IslandManager.FindParticlesInIslands(IslandValidationIslands, OUT IslandValidationIslandParticles);
	}

	/** Get earliest frame we are allowed to rewind to taking these three into account:
	* - Earliest frame in history
	* - Blocked resim frame
	* - Setting for how many frames behind the requested frame we allow to search (NOTE: -1 because the for-loop is excluding the EarliestFrame) */
	const int32 FramesBehindSettings = FMath::Max(FMath::Max(FindValidFrameNFramesBehind, 0), FMath::CeilToInt32(static_cast<float>(Capacity()) * FMath::Clamp(FindValidFrameNPercentBehind, 0.0f, 1.0f)));
	const int32 EarliestFrame = FMath::Max3(GetEarliestFrame_Internal(), BlockResimFrame, RequestedFrame - 1 - FramesBehindSettings);
	bool bHasTargetHistory = false;
	bool bHasCustomDataHistory = false;

	for (int32 CheckFrame = RequestedFrame; CheckFrame > EarliestFrame; CheckFrame--)
	{
		const FFrameAndPhase FrameAndPhase{ CheckFrame, FFrameAndPhase::PrePushData };
		bHasTargetHistory = true;
		bHasCustomDataHistory = true;
		
#if DEBUG_REWIND_DATA
		UE_LOGF(LogChaos, Log, "CLIENT | PT | FindValidResimFrame | Processing resim particles | Check Frame: %d | Total Particle Count: %d | ResimIslands Particle Count: %d | ResimFrameValidation: %d | ValidTargetFrame: %d", CheckFrame, DirtyParticles.Num(), IslandValidationIslandParticles.Num(), ResimFrameValidation, ValidTargetFrame);
#endif

		if (ResimFrameValidation == static_cast<int32>(EResimFrameValidation::IslandValidation))
		{
			// Iterate over islands previously found having resim particles in them and check if the particles in the islands have targets
			for (const FGeometryParticleHandle* IslandParticle : IslandValidationIslandParticles)
			{
				// Cache particle handles for objects in islands that need resim
				if (FDirtyParticleInfo* DirtyParticleInfo = FindDirtyObj(*IslandParticle))
				{
					if (!TargetFinderHelper(DirtyParticleInfo, FrameAndPhase))
					{
						bHasTargetHistory = false;
						break;
					}

					if (!CustomDataFinderHelper(DirtyParticleInfo, FrameAndPhase))
					{
						bHasCustomDataHistory = false;
						break;
					}
				}
			}
		}
		else
		{
			for (FDirtyParticleInfo& DirtyParticleInfo : DirtyParticles)
			{
				// If running validation leniency, check if the particle is marked for resimulation else don't bother checking for valid target states.
				if (ResimFrameValidation == static_cast<int32>(EResimFrameValidation::TriggerParticleValidation))
				{
					FGeometryParticleHandle* Handle = DirtyParticleInfo.GetObjectPtr();
					if(IslandManager.GetParticleResimFrame(Handle) == INDEX_NONE)
					{
						continue;
					}
				}

				if (!TargetFinderHelper(&DirtyParticleInfo, FrameAndPhase))
				{
					bHasTargetHistory = false;
					break;
				}

				if (!CustomDataFinderHelper(&DirtyParticleInfo, FrameAndPhase))
				{
					bHasCustomDataHistory = false;
					break;
				}
			}
		}

		if (bHasTargetHistory && bHasCustomDataHistory)
		{
			ValidFrame = CheckFrame;
			break;
		}
		else if (bHasTargetHistory && ValidTargetFrame == INDEX_NONE)
		{
			// If we have a valid frame with targets from the server but no custom data, cache the frame number to use if we don't find any frame with valid custom data
			ValidTargetFrame = CheckFrame;
		}
	}
	
	// Check if no valid frame was found with both target state and custom data
	if (ValidFrame == INDEX_NONE)
	{
		// Check if a valid target frame was found
		if (ValidTargetFrame != INDEX_NONE)
		{
			ValidFrame = ValidTargetFrame;
		}
		else // No valid frame found
		{
			// Clamp the incomplete-history fallback, if outside of rewind history return an invalid resim frame (INDEX_NONE)
			ValidFrame = (bResimIncompleteHistory && IsFrameWithinRewindHistory(RequestedFrame)) ? RequestedFrame : INDEX_NONE;

			// If we can't perform a resim, clear bNeedsResim flags on dirty physics objects, else they will get cleared during AdvanceFrame if we are resimulating
			for (FDirtyParticleInfo& DirtyParticleInfo : DirtyParticles)
			{
				DirtyParticleInfo.bNeedsResim = false;
			}
			for (FDirtyJointInfo& DirtyJointInfo : DirtyJoints)
			{
				DirtyJointInfo.bNeedsResim = false;
			}

	#if DEBUG_REWIND_DATA
			UE_LOGF(LogChaos, Warning, "CLIENT | PT | FindValidResimFrame | No valid resim frame found | RequestedFrame: %d | ValidFrame: %d | ValidTargetFrame: %d | EarliestFrame: %d | HasTargetHistory: %d | HasCustomDataHistory: %d | EarliestHistoryFrame: %d | CurrentFrame: %d | FramesSaved: %d | ResimFrameValidation: %d", RequestedFrame, ValidFrame, ValidTargetFrame, EarliestFrame, bHasTargetHistory, bHasCustomDataHistory, GetEarliestFrame_Internal(), CurrentFrame(), FramesSaved, ResimFrameValidation);
	#endif
		}
	}

	return ValidFrame;
}

void FRewindData::PushStateAtFrame(FGeometryParticleHandle& Handle, int32 Frame, FFrameAndPhase::EParticleHistoryPhase Phase, 
	const FVector& Position, const FQuat& Quaternion, const FVector& LinVelocity, const FVector& AngVelocity, const bool bShouldSleep)
{
	const bool bRecordingHistory = !IsResimAndInSync(Handle);

	FDirtyParticleInfo& Info = FindOrAddDirtyObj(Handle);
	FGeometryParticleStateBase& History = Info.GetHistory();
	const FFrameAndPhase FrameAndPhase{ Frame, Phase };
	LatestTargetFrame = bRecordingHistory ? FMath::Max(LatestTargetFrame, Frame) : LatestTargetFrame;

	if (bRecordingHistory || History.TargetPositions.IsClean(FrameAndPhase))
	{
		FParticlePositionRotation& PositionRotation = History.TargetPositions.Insert(FrameAndPhase, PropertiesPool);
		PositionRotation.SetX(Position);
		PositionRotation.SetR(Quaternion);
	}

	if (bRecordingHistory || History.TargetVelocities.IsClean(FrameAndPhase))
	{
		FParticleVelocities& Velocities = History.TargetVelocities.Insert(FrameAndPhase, PropertiesPool);
		Velocities.SetV(LinVelocity);
		Velocities.SetW(AngVelocity);
	}

	if (bRecordingHistory || History.TargetStates.IsClean(FrameAndPhase))
	{
		FParticleDynamicMisc& DynamicsMisc = History.TargetStates.Insert(FrameAndPhase, PropertiesPool);
		DynamicsMisc.SetObjectState(bShouldSleep ? EObjectStateType::Sleeping : EObjectStateType::Dynamic);
		DynamicsMisc.SetDisabled(false);
	}
}

void FRewindData::ProcessDirtyKinematicTargets(const TParticleView<TPBDRigidParticles<FReal, 3>>& ActiveKinematicParticles)
{
	// Cache the kinematic target of moving kinematic objects
	for (FTransientPBDRigidParticleHandle& ActiveKinematicParticle : ActiveKinematicParticles)
	{
		CacheKinematicTarget(*ActiveKinematicParticle.Handle());
	}
}

void FRewindData::ProcessDirtyPTParticles(const TParticleView<TPBDRigidParticles<FReal, 3>>&  DirtyPTParticles)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (Solver)
	{
		Solver->FinalizeRewindData(DirtyPTParticles);
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Cache dirty particles from simulation
	int32 DataIdx = 0;
	for (TPBDRigidParticleHandleImp<FReal, 3, false>& DirtyPTParticle : DirtyPTParticles)
	{
		PushPTDirtyData(*DirtyPTParticle.Handle(), DataIdx++);
	}
}

void FRewindData::PushPTDirtyData(TPBDRigidParticleHandle<FReal, 3>& Handle, const int32 SrcDataIdx)
{
	if (bRewindDataOptimization)
	{
		/** When using optimization, mark particle as dirty and then rely on FRewindData::CacheCurrentDirtyData() to cache PrePushData 
		* If this is a new entry in the dirty particles collection, cache now since CacheCurrentDirtyData for PrePushData has already happened this frame */

		FDirtyParticleInfo* Info = FindDirtyObj(Handle);
		if (!Info || Info->InitializedOnStep == CurFrame)
		{
			CacheDirtyParticleData(&Handle, FFrameAndPhase::PostCallbacks, /*bDirty*/true);
		}
		else
		{
			// Mark particle as dirty so it doesn't get cleared from the dirty particles
			Info->MarkDirty(CurFrame);
		}
		return;
	}

	const bool bRecordingHistory = !IsResimAndInSync(Handle);

	FDirtyParticleInfo& Info = FindOrAddDirtyObj(Handle);
	Info.MarkDirty(CurFrame);
	FGeometryParticleStateBase& Latest = Info.GetHistory();

	const FFrameAndPhase FrameAndPhase{ CurFrame, FFrameAndPhase::PostCallbacks };

	if (bRecordingHistory || Latest.ParticlePositionRotation.IsClean(FrameAndPhase))
	{
		if (FParticlePositionRotation* PreXR = Latest.ParticlePositionRotation.WriteAccessNonDecreasing(FrameAndPhase, PropertiesPool))
		{
			/** This is called post-solve but before PQ are applied on XR
			* If this is a kinematic moving object XR have been updated already in the integrate step via KinematicTarget and the velocity has been updated based on the XR change
			* Get the pre-solve state of moving kinematic particles by stepping their XR back one step via their velocities. */
			if (Handle.IsMovingKinematic())
			{
				PreXR->SetX(Handle.GetX() - (Handle.GetV() * Solver->GetLastDt()));
				PreXR->SetR(Chaos::FRotation3::IntegrateRotationWithAngularVelocity(Handle.GetR(), Handle.GetWf(), -Solver->GetLastDt()));
			}
			else
			{
				PreXR->CopyFrom(Handle);
			}
		}
	}

	if (bRecordingHistory || Latest.Velocities.IsClean(FrameAndPhase))
	{
		if (FParticleVelocities* PreVelocities = Latest.Velocities.WriteAccessNonDecreasing(FrameAndPhase, PropertiesPool))
		{
			PreVelocities->SetV(Handle.GetPreV());
			PreVelocities->SetW(Handle.GetPreW());
		}
	}
	
	if (bRecordingHistory || Latest.DynamicsMisc.IsClean(FrameAndPhase))
	{
		if (FParticleDynamicMisc* PreDynamicMisc = Latest.DynamicsMisc.WriteAccessNonDecreasing(FrameAndPhase, PropertiesPool))
		{
			PreDynamicMisc->CopyFrom(Handle);	//everything is immutable except object state
			PreDynamicMisc->SetObjectState(Handle.PreObjectState());
		}
	}
}

void FRewindData::CacheKinematicTarget(TPBDRigidParticleHandle<FReal, 3>& Handle)
{
	if (!bRewindDataOptimization)
	{
		return;
	}

	FKinematicGeometryParticleHandle* Kinematic = Handle.CastToKinematicParticle();
	if (!Kinematic || Kinematic->KinematicTarget().GetMode() == EKinematicTargetMode::None)
	{
		return;
	}

	const FFrameAndPhase FrameAndPhase{ CurFrame, FFrameAndPhase::PrePushData };

	FDirtyParticleInfo& Info = FindOrAddDirtyObj(Handle);
	Info.MarkDirty(CurFrame);

	if (IsResim())
	{
		if (!Info.bResimAsFollower)
		{
			// Check if particle differ from the currently cached history for this frame, if so, clear the history and mark particle as desynced so it will cache data during resimulation
			DesyncIfNecessary</*bSkipDynamics=*/false>(Info, FrameAndPhase);
		}

		if (Kinematic->SyncState() == ESyncState::InSync)
		{
			/* No need to cache data in history if the particle is still in sync during resimulation */
			return;
		}
	}

	FGeometryParticleStateBase& Latest = Info.GetHistory();

	const bool bRecordingHistory = !IsResimAndInSync(Handle);
	if (bRecordingHistory)
	{
		if (auto* Data = Latest.KinematicTarget.WriteAccessNonDecreasing(FrameAndPhase, PropertiesPool))
		{
			CopyDataFromObject(*Data, *Kinematic);
		}
	}
}

void FRewindData::CacheDirtyParticleData(FGeometryParticleHandle* Particle, const FFrameAndPhase::EParticleHistoryPhase& CurrentPhase, const bool& bDirty)
{
	// If we are resimulating, only process particles under EResimType::FullResim
	if (IsResim() && Particle->ResimType() != EResimType::FullResim)
	{
		return;
	}

	FDirtyParticleInfo& DirtyParticleInfo = FindOrAddDirtyObj(*Particle);
	if (bDirty)
	{
		DirtyParticleInfo.MarkDirty(CurFrame);
	}

	if (DirtyParticleInfo.InitializedOnStep > CurFrame)
	{
		// Don't cache data for uninitialized particle
		return;
	}
	else if (DirtyParticleInfo.InitializedOnStep == CurFrame && CurrentPhase < FFrameAndPhase::PostCallbacks)
	{
		/** Don't cache data in the PrePushData or PostPushData phase on the frame the particle gets initialized
		* Do cache the data in the PostCallbacks though, with optimization we will convert it from PostCallback state into PrePushData and store it there to have the initial state of the particle */
		return;
	}

	// When using optimization we always cache data in the PrePushData phase
	const FFrameAndPhase FrameAndPhase{ CurFrame, bRewindDataOptimization ? FFrameAndPhase::PrePushData : CurrentPhase };

	if (IsResim())
	{
		// NOTE, we have already checked that the particle is EResimType::FullResim during resimulation
		{
			// Check if particle differ from the currently cached history for this frame, if so, clear the history and mark particle as desynced so it will cache data during resimulation
			DesyncIfNecessary</*bSkipDynamics=*/false>(DirtyParticleInfo, FrameAndPhase);
		}

		if (Particle->SyncState() == ESyncState::InSync)
		{
			/* No need to cache data in history if the particle is still in sync during resimulation */
			return;
		}
	}

	const bool bCachePreFromPost = bRewindDataOptimization && CurrentPhase == FFrameAndPhase::PostCallbacks;

	auto DirtyPropHelper = [this, FrameAndPhase](auto& Property, const auto& Particle, const auto& CacheFunction)
		{
			const bool bRecordingHistory = !IsResimAndInSync(Particle);

			if (bRecordingHistory || Property.IsClean(FrameAndPhase))
			{
				if (auto* Data = Property.WriteAccessNonDecreasing(FrameAndPhase, PropertiesPool))
				{
					CacheFunction(Particle, *Data);
				}
			}
		};

	FKinematicGeometryParticleHandle* KinematicParticle = Particle->CastToKinematicParticle();
	FPBDRigidParticleHandle* RigidParticle = KinematicParticle ? KinematicParticle->CastToRigidParticle() : nullptr;;

	FGeometryParticleStateBase& Latest = DirtyParticleInfo.GetHistory();

	DirtyPropHelper(Latest.ParticlePositionRotation, *Particle, [this, &bCachePreFromPost, &RigidParticle](const auto& Particle, FParticlePositionRotation& Data)
	{
		if (bCachePreFromPost && RigidParticle && RigidParticle->IsMovingKinematic())
		{
			/** FFrameAndPhase::PostCallbacks is called at the end of the physics solve
			* If this is a kinematic moving object then XR have been updated already in the integrate step via KinematicTarget and the velocity has been updated based on the XR change
			* Get the pre-solve state by stepping XR back once via the velocities. */

			Data.SetX(RigidParticle->GetX() - (RigidParticle->GetV() * Solver->GetLastDt()));
			Data.SetR(Chaos::FRotation3::IntegrateRotationWithAngularVelocity(RigidParticle->GetR(), RigidParticle->GetWf(), -Solver->GetLastDt()));
		}
		else
		{
			// This works during PostCallback since XR is not yet updated from PQ even if it's actually called PostSolve instead of PostCallback
			CopyDataFromObject(Data, Particle); 
		}
	});

	DirtyPropHelper(Latest.NonFrequentData, *Particle, [this](const auto& Particle, FParticleNonFrequentData& Data)
	{
		CopyDataFromObject(Data, Particle);
	});

	if (KinematicParticle)
	{
		DirtyPropHelper(Latest.Velocities, *KinematicParticle, [this, &bCachePreFromPost, &RigidParticle](const auto& Particle, FParticleVelocities& Data)
		{
			if (bCachePreFromPost && RigidParticle)
			{
				Data.SetV(RigidParticle->GetPreV());
				Data.SetW(RigidParticle->GetPreW());
			}
			else
			{
				CopyDataFromObject(Data, Particle);
			}
		});

		if (!bRewindDataOptimization)
		{
			// Cache kinematic target here if not running with optimization, else it's cached in CacheKinematicTarget()
			DirtyPropHelper(Latest.KinematicTarget, *KinematicParticle, [this](const auto& Particle, FKinematicTarget& Data)
			{
				CopyDataFromObject(Data, Particle);
			});
		}

		if (RigidParticle)
		{
			if (FrameAndPhase.Phase != FFrameAndPhase::PrePushData)
			{
				// Don't cache dynamics for PrePushData, it's always zeroed out there. 
				DirtyPropHelper(Latest.Dynamics, *RigidParticle, [this, &bCachePreFromPost](const auto& Particle, FParticleDynamics& Data)
				{
					CopyDataFromObject(Data, Particle); 
				});
			}

			DirtyPropHelper(Latest.DynamicsMisc, *RigidParticle, [this, &bCachePreFromPost](const auto& Particle, FParticleDynamicMisc& Data)
			{
				CopyDataFromObject(Data, Particle);
				if (bCachePreFromPost)
				{
					/** FFrameAndPhase::PostCallbacks is called at the end of the physics solve, get the object state from before the solve */
					Data.SetObjectState(Particle.PreObjectState());
				}
			});

			DirtyPropHelper(Latest.MassProps, *RigidParticle, [this](const auto& Particle, FParticleMassProps& Data)
			{
				CopyDataFromObject(Data, Particle);
			});
		}
	}
}

void FRewindData::CacheDirtyJointData(FPBDJointConstraintHandle* Joint, const FFrameAndPhase::EParticleHistoryPhase& CurrentPhase, const bool& bDirty)
{
	FDirtyJointInfo& DirtyJointInfo = FindOrAddDirtyObj(*Joint);
	if (bDirty)
	{
		DirtyJointInfo.MarkDirty(CurFrame);
	}

	if (DirtyJointInfo.InitializedOnStep > CurFrame)
	{
		// Don't cache data for uninitialized joint
		return;
	}
	else if (DirtyJointInfo.InitializedOnStep == CurFrame && CurrentPhase < FFrameAndPhase::PostCallbacks)
	{
		/** Don't cache data in the PrePushData or PostPushData phase on the frame the joint gets initialized
		* Do cache the data in the PostCallbacks though, with optimization we will convert it from PostCallback state into PrePushData and store it there to have the initial state of the particle */
		return;
	}

	// When using optimization we always cache data in the PrePushData phase
	const FFrameAndPhase FrameAndPhase{ CurFrame, bRewindDataOptimization ? FFrameAndPhase::PrePushData : CurrentPhase };

	auto DirtyPropHelper = [this, FrameAndPhase](auto& Property, const auto& Object)
		{
			const bool bRecordingHistory = !IsResimAndInSync(Object);

			if (bRecordingHistory || Property.IsClean(FrameAndPhase))
			{
				if (auto* Data = Property.WriteAccessNonDecreasing(FrameAndPhase, PropertiesPool))
				{
					CopyDataFromObject(*Data, Object);
				}
			}
		};

	if (IsResim())
	{
		if (!DirtyJointInfo.bResimAsFollower)
		{
			// Check if joint differ from the currently cached history for this frame, if so, clear the history and mark joint as desynced so it will cache data during resimulation
			DesyncIfNecessary</*bSkipDynamics=*/false>(DirtyJointInfo, FrameAndPhase);
		}

		if (Joint->SyncState() == ESyncState::InSync)
		{
			/* No need to cache data in history if the particle is still in sync during resimulation */
			return;
		}
	}

	FJointStateBase& Latest = DirtyJointInfo.GetHistory();

	DirtyPropHelper(Latest.JointSettings, *Joint);
}

void FRewindData::CacheCurrentDirtyData(const FFrameAndPhase::EParticleHistoryPhase& CurrentPhase)
{
	QUICK_SCOPE_CYCLE_COUNTER(Chaos_CacheCurrentDirtyData);

	if (!bRewindDataOptimization)
	{
		// The non optimized flow cache data in various other callbacks which will conflict with this
		return;
	}

	const bool bIsResim = IsResim();
	for (FDirtyParticleInfo& DirtyParticleInfo : DirtyParticles)
	{
		if (FGeometryParticleHandle* Geometry = DirtyParticleInfo.GetObjectPtr())
		{
			// During resim, only FullResim particles need caching skip ResimAsFollower and FrozenDuringResim
			// to avoid unnecessary function calls and FindOrAddDirtyObj hash lookups
			if (bIsResim && Geometry->ResimType() != EResimType::FullResim)
			{
				continue;
			}
			CacheDirtyParticleData(Geometry, CurrentPhase, /*bDirty*/false);
		}
	}

	for (FDirtyJointInfo& DirtyJointInfo : DirtyJoints)
	{
		if (FPBDJointConstraintHandle* Joint = DirtyJointInfo.GetObjectPtr())
		{
			CacheDirtyJointData(Joint, CurrentPhase, /*bDirty*/false);
		}
	}
}

FGeometryParticleState FRewindData::GetPastStateAtFrame(const FGeometryParticleHandle& Handle, int32 Frame, FFrameAndPhase::EParticleHistoryPhase Phase) const
{
	return GetPastStateAtFrameImp<FGeometryParticleState>(DirtyParticles, Handle, Frame, bRewindDataOptimization ? FFrameAndPhase::EParticleHistoryPhase::PrePushData : Phase);
}

FJointState FRewindData::GetPastJointStateAtFrame(const FPBDJointConstraintHandle& Handle, int32 Frame, FFrameAndPhase::EParticleHistoryPhase Phase) const
{
	return GetPastStateAtFrameImp<FJointState>(DirtyJoints, Handle, Frame, bRewindDataOptimization ? FFrameAndPhase::EParticleHistoryPhase::PrePushData : Phase);
}

CHAOS_API int32 InterpolateTargetGaps = 5;
FAutoConsoleVariableRef CVarResimInterpolateTargetGaps(TEXT("p.Resim.InterpolateTargetGaps"), InterpolateTargetGaps, TEXT("How many frame gaps in replicated targets we should fill by interpolating between the previous and the new target received. Value in max number of frames to interpolate, deactivate by setting to 0."));

float AveragePredictedFramesScalar = 0.2f;
static FAutoConsoleVariableRef CVarResimAveragePredictedFramesScalar(TEXT("p.Resim.AveragePredictedFramesScalar"), AveragePredictedFramesScalar, TEXT("Multiplier for how quickly the average number of predicted frames are updated when changed, clamped between 0.01 and 1.0."));

void FRewindData::SetTargetStateAtFrame(FGeometryParticleHandle& Handle, const int32 Frame, FFrameAndPhase::EParticleHistoryPhase Phase,
	const FVector& Position, const FQuat& Quaternion, const FVector& LinVelocity, const FVector& AngVelocity, const bool bShouldSleep)
{
	if (InterpolateTargetGaps > 0)
	{
		FDirtyParticleInfo& Info = FindOrAddDirtyObj(Handle);
		FGeometryParticleStateBase& History = Info.GetHistory();
		FFrameAndPhase FrameAndPhase;

		if (History.TargetPositions.GetHeadFrameAndPhase(FrameAndPhase))
		{
			const int32 FrameDiff = Frame - FrameAndPhase.Frame;
			if (FrameDiff > 1 && FrameDiff <= InterpolateTargetGaps)
			{
				const FParticlePositionRotation* TargetXR = History.TargetPositions.Read(FrameAndPhase, PropertiesPool);
				const FParticleVelocities* TargetVW = History.TargetVelocities.Read(FrameAndPhase, PropertiesPool);
				const FParticleDynamicMisc* TargetDynamic = History.TargetStates.Read(FrameAndPhase, PropertiesPool);
				if (TargetXR && TargetVW && TargetDynamic)
				{
					// Copy the previous state to lerp from
					const FVector PrevX = TargetXR->GetX();
					const FQuat PrevR = TargetXR->GetR();
					const FVector PrevV = TargetVW->GetV();
					const FVector PrevW = TargetVW->GetW();
					const bool bSleep = bShouldSleep && TargetDynamic->ObjectState() == EObjectStateType::Sleeping;

					for (int32 InterpFrame = 1; InterpFrame < FrameDiff; InterpFrame++)
					{
						const float Alpha = static_cast<float>(InterpFrame) / static_cast<float>(FrameDiff);

						PushStateAtFrame(Handle, FrameAndPhase.Frame + InterpFrame, Phase,
							FMath::Lerp(PrevX, Position, Alpha),
							FRotation3::Slerp(PrevR, Quaternion, Alpha),
							FMath::Lerp(PrevV, LinVelocity, Alpha),
							FMath::Lerp(PrevW, AngVelocity, Alpha),
							bSleep);
					}
				}
			}
		}
	}

	PushStateAtFrame(Handle, Frame, Phase, Position, Quaternion, LinVelocity, AngVelocity, bShouldSleep);

	// Cache an average of how many frames we are currently forward predicted
	if (!IsResim() && Frame > 0)
	{
		const float PredictedFrames = static_cast<float>(LatestFrame - Frame);
		NumPredictedFramesAverage += (PredictedFrames - NumPredictedFramesAverage) * FMath::Clamp(AveragePredictedFramesScalar, 0.01f, 1.0f);
	}
}

void FRewindData::ForceResimAsFollower(FGeometryParticleHandle* Handle, bool bForceResim)
{
	if (bForceResim)
	{
		ParticlesToForceResimAsFollower.Add(Handle);
	}
	else
	{
		ParticlesToForceResimAsFollower.Remove(Handle);
	}
}

int32 FRewindData::GetLatestReceivedTargetStateFrame(Chaos::FGeometryParticleHandle& Handle) const
{
	if (const FDirtyParticleInfo* DirtyParticleInfo = FindDirtyObj(Handle))
	{
		FFrameAndPhase FrameAndPhase;
		if (DirtyParticleInfo->GetHistory().TargetPositions.GetHeadFrameAndPhase(FrameAndPhase))
		{
			return FrameAndPhase.Frame;
		}
	}
	return INDEX_NONE;
}

void FRewindData::RequestResimulation(int32 Frame, Chaos::FGeometryParticleHandle* Particle)
{
	if (IsResim())
	{
		return;
	}

	// Update ResimFrame but don't allow to set a newer frame than already set
	SetResimFrame((ResimFrame == INDEX_NONE) ? Frame : FMath::Min(ResimFrame, Frame));

	if (Particle && ensure(Solver))
	{
		if (FDirtyParticleInfo* DirtyParticleInfo = FindDirtyObj(*Particle))
		{
			DirtyParticleInfo->bNeedsResim = true;
		}

		if (Chaos::FPBDRigidsEvolution* Evolution = Solver->GetEvolution())
		{
			Evolution->GetIslandManager().SetParticleResimFrame(Particle, ResimFrame);
		}
	}
}

int32 ResimClampedFramesOverride = 0;
FAutoConsoleVariableRef CVarResimClampedFramesOverride(TEXT("p.Resim.ClampedFramesOverride"), ResimClampedFramesOverride, TEXT("Override how far behind current frame is considered the earliest available resim frame. Gets clamped between 0 and number of frames cached in history."));

int32 FRewindData::GetEarliestFrame_Internal() const
{
	checkf(ResimClampedFrames > INDEX_NONE, TEXT("FRewindData not initialized correctly, ensure use of proper constructor or Init() function calls"));

	const int32 EffectiveResimClamp = FMath::Min((ResimClampedFramesOverride > 0) ? ResimClampedFramesOverride : ResimClampedFrames, FramesSaved);
	return LatestFrame - EffectiveResimClamp;
}

void FRewindData::BlockResim()
{
	if (LatestFrame > BlockResimFrame)
	{
		BlockResimFrame = LatestFrame;
	}
}

}
