// Copyright Epic Games, Inc. All Rights Reserved.

#include "TaskDispatcherSolver.h"

#include "PhysicsProxy/ClusterUnionPhysicsProxy.h"
#include "PhysicsProxy/GeometryCollectionPhysicsProxy.h"


namespace Chaos::Private
{

void FTaskDispatcherSolver::Initialize(TParticleView<FPBDRigidParticles>& DirtyParticles)
{
	using TSOA = typename TParticleView<FPBDRigidParticles>::TSOA;
	using THandle = typename TSOA::THandleType;
	using THandleBase = typename THandle::THandleBase;
	using TTransientHandle = typename THandle::TTransientHandle;
	int32 DispatchBatchIndex = 0;

	// Precompute NumRigidBatches 
	NumRigidBatches = 0;
	for (int32 ViewIndex = 0; ViewIndex < DirtyParticles.SOAViews.Num(); ++ViewIndex)
	{
		const TSOAView<TSOA>& SOAView = DirtyParticles.SOAViews[ViewIndex];
		const int32 NumParticles = SOAView.Size();
		if (NumParticles == 0)
		{
			continue;
		}
		constexpr int32 MinRigidPerTask = 50;
		const int32 NumTasks = FMath::Max(FMath::Min(FTaskGraphInterface::Get().GetNumWorkerThreads() - 1, Chaos::MaxNumWorkers), 1);
		const int32 NumRigidTask = FMath::Max(FMath::Min(NumTasks, NumParticles), 1);
		const int32 DirtyByTask = FMath::Max(FMath::DivideAndRoundUp(NumParticles, NumRigidTask), MinRigidPerTask);
		NumRigidBatches += FMath::DivideAndRoundUp(NumParticles, DirtyByTask);
	}

	DirtyRigidPendingTasks.Reset(NumRigidBatches);
	DirtyGCAndClusterPendingTask = UE::Tasks::FTask();

	ActiveRigidArray.SetNum(NumRigidBatches);

	ActiveGC.Reset();
	ActiveClusterUnions.Reset();

	PullDataDirtyRigidsArray.Reset();

	PreviousCopyRigidTask = UE::Tasks::FTask();
}

void FTaskDispatcherSolver::BufferRigidResults(FPullPhysicsData* PullData)
{
	// Num Batch should be the same for the three arrays
	check(ActiveRigidArray.Num() == NumRigidBatches);
	check(ActiveRigidArray.Num() == DirtyRigidPendingTasks.Num());

	PullDataDirtyRigidsArray.SetNum(NumRigidBatches);
	//we only fill this once per frame
	ensure(PullData->DirtyRigids.Num() == 0);

	for (int32 BatchIndex = 0; BatchIndex < NumRigidBatches; BatchIndex++)
	{
		TArray<FDirtyRigidParticleData>& PullDataDirtyRigids = PullDataDirtyRigidsArray[BatchIndex];
		PullDataDirtyRigids.Reset();
		TArray<FSingleParticlePhysicsProxy*>& ActiveRigid = ActiveRigidArray[BatchIndex];
		UE::Tasks::FTask RigidPendingTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [&ActiveRigid, &PullDataDirtyRigids]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_BufferRigidResultsTasks);
			for (int32 Idx = 0; Idx < ActiveRigid.Num(); ++Idx)
			{
				PullDataDirtyRigids.AddDefaulted();
				ActiveRigid[Idx]->BufferPhysicsResults(PullDataDirtyRigids.Last());
			}
		}, DirtyRigidPendingTasks[BatchIndex]);

		// Link the following task with the one from the previous iteration to keep determinism
		UE::Tasks::FTask RigidCopyPendingTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [PullData, &PullDataDirtyRigids]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_BufferRigidResultsCopyTasks);
			PullData->DirtyRigids.Append(PullDataDirtyRigids);
		}, TArray<UE::Tasks::FTask>{RigidPendingTask, PreviousCopyRigidTask});
		PreviousCopyRigidTask = RigidCopyPendingTask;
	}
}

void FTaskDispatcherSolver::BufferGCResults(FPullPhysicsData* PullData, FPBDRigidsSolver* RigidsSolver)
{
	GCPendingTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, PullData, RigidsSolver]()
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_BufferGCResultsTask);

		// This ActiveGC cannot be read while the last DirtyGCAndClusterPendingTask is not finished
		// that why we need to have this code in a dependent task
		int32 NumGC = ActiveGC.Num();
		constexpr int32 MinGCPerTask = 1;
		const int32 NumTasks = FMath::Max(FMath::Min(FTaskGraphInterface::Get().GetNumWorkerThreads() - 1, Chaos::MaxNumWorkers), 1);
		const int32 NumGCTask = FMath::Max(FMath::Min(NumTasks, NumGC), 1);
		const int32 GCByTask = FMath::Max(FMath::DivideAndRoundUp(NumGC, NumGCTask), MinGCPerTask);
		const int32 NumGCBatches = FMath::DivideAndRoundUp(NumGC, GCByTask);

		//we only fill this once per frame
		ensure(PullData->DirtyGeometryCollections.Num() == 0);
		PullData->DirtyGeometryCollections.SetNum(NumGC);

		GCBatchTasks.SetNum(NumGCBatches);
		for (int32 BatchIndex = 0; BatchIndex < NumGCBatches; BatchIndex++)
		{
			const int32 StartIndex = BatchIndex * GCByTask;
			int32 EndIndex = (BatchIndex + 1) * GCByTask;
			EndIndex = FMath::Min(NumGC, EndIndex);

			GCBatchTasks[BatchIndex] = UE::Tasks::Launch(UE_SOURCE_LOCATION, [StartIndex, EndIndex, RigidsSolver, &ActiveGC = ActiveGC, &DirtyGeometryCollections = PullData->DirtyGeometryCollections]()
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_BufferGCResultsTasks);
				for (int32 Idx = StartIndex; Idx < EndIndex; ++Idx)
				{
					ActiveGC[Idx]->BufferPhysicsResults_Internal(RigidsSolver, DirtyGeometryCollections[Idx]);
				}
			});
		}
	}, DirtyGCAndClusterPendingTask);
}


void FTaskDispatcherSolver::BufferClusterResults(FPullPhysicsData* PullData)
{
	ClusterPendingTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, PullData]()
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_BufferClusterResultsTask);

		// This value cannot be read while the last DirtyGCAndClusterPendingTask is not finished
		// that why we need to have this code in a dependent task
		int32 NumClusters = ActiveClusterUnions.Num();
		constexpr int32 MinClusterPerTask = 1;
		const int32 NumTasks = FMath::Max(FMath::Min(FTaskGraphInterface::Get().GetNumWorkerThreads() - 1, Chaos::MaxNumWorkers), 1);
		const int32 NumClusterTask = FMath::Max(FMath::Min(NumTasks, NumClusters), 1);
		const int32 ClusterByTask = FMath::Max(FMath::DivideAndRoundUp(NumClusters, NumClusterTask), MinClusterPerTask);
		const int32 NumClusterBatches = FMath::DivideAndRoundUp(NumClusters, ClusterByTask);

		//we only fill this once per frame
		ensure(PullData->DirtyClusterUnions.Num() == 0);
		PullData->DirtyClusterUnions.SetNum(NumClusters);

		ClusterBatchTasks.SetNum(NumClusterBatches);
		for (int32 BatchIndex = 0; BatchIndex < NumClusterBatches; BatchIndex++)
		{
			const int32 StartIndex = BatchIndex * ClusterByTask;
			int32 EndIndex = (BatchIndex + 1) * ClusterByTask;
			EndIndex = FMath::Min(NumClusters, EndIndex);

			ClusterBatchTasks[BatchIndex] = UE::Tasks::Launch(UE_SOURCE_LOCATION, [StartIndex, EndIndex, &ActiveClusterUnions = ActiveClusterUnions, &DirtyClusterUnions = PullData->DirtyClusterUnions]()
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_BufferClusterResultsTasks);
				for (int32 Idx = StartIndex; Idx < EndIndex; ++Idx)
				{
					ActiveClusterUnions[Idx]->BufferPhysicsResults_Internal(DirtyClusterUnions[Idx]);
				}
			}, GCBatchTasks);
		}
		}, GCPendingTask);
}

void FTaskDispatcherSolver::WaitTaskEndBufferResults() const
{
	check(ActiveRigidArray.Num() == NumRigidBatches);
	check(ActiveRigidArray.Num() == DirtyRigidPendingTasks.Num());

	// First we need to wait for the outer task which will create inner task
	UE::Tasks::Wait(TArray<UE::Tasks::FTask>{GCPendingTask});
	// We are creating a dependency between GC and CU threads to be sure if any dependency is created
	// a CU and its GC the program doesn't create any race condition
	UE::Tasks::Wait(TArray<UE::Tasks::FTask>{ClusterPendingTask});
	// Only when those outer tasks are finished and so those inner task created we can wait for those inner tasks
	TArray<UE::Tasks::FTask> TaskToWaitOn({ PreviousCopyRigidTask });
	TaskToWaitOn.Append(GCBatchTasks);
	TaskToWaitOn.Append(ClusterBatchTasks);
	UE::Tasks::Wait(TaskToWaitOn);
}

}