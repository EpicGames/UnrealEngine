// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "Tasks/Task.h"


namespace Chaos
{
	class FClusterUnionPhysicsProxy;
}
class FGeometryCollectionPhysicsProxy;


namespace Chaos::Private
{

// This class is responsible to dispatch the solver tasks
// It orchestrates the task dependencies according to data flow.
// This class shouldn't do any simulation logic and code.
class FTaskDispatcherSolver
{
private:

	// Array of batch of array of rigid particles
	TArray<TArray<FSingleParticlePhysicsProxy*>> ActiveRigidArray;
	// Geometry Collection array
	TArray<FGeometryCollectionPhysicsProxy*> ActiveGC;
	TArray<FClusterUnionPhysicsProxy*> ActiveClusterUnions;

	// Tasks that collect rigid particle
	TArray<UE::Tasks::FTask> DirtyRigidPendingTasks;
	// Tasks that collect Geometry collection and cluster union
	UE::Tasks::FTask DirtyGCAndClusterPendingTask;

	// Outer tasks which will create sub task once the collect task will be finish
	// collecting Geometry collection and cluster union proxies 
	UE::Tasks::FTask GCPendingTask;
	UE::Tasks::FTask ClusterPendingTask;

	// Last task which copy in PullData the proxy for those three types of proxy
	UE::Tasks::FTask PreviousCopyRigidTask;

	// PullData copies to buffer physics results by batch
	TArray<TArray<FDirtyRigidParticleData>> PullDataDirtyRigidsArray;
	
	// GC and cluster batches tasks launches by GCPendingTask & ClusterPendingTask
	TArray<UE::Tasks::FTask> GCBatchTasks;
	TArray<UE::Tasks::FTask> ClusterBatchTasks;

	int32 NumRigidBatches;

public:
	
	int32 GetNumRigidBatches() const { return NumRigidBatches; }

	FTaskDispatcherSolver()
	{
	}

	void Initialize(TParticleView<FPBDRigidParticles>& DirtyParticles);

	TArray<FGeometryCollectionPhysicsProxy*>& GetActiveGC()
	{
		return ActiveGC;
	}

	template<typename LambdaRigid, typename LambdaGCAndCluster>
	int32 CollectPhysicsResults(LambdaRigid CollectRigidResults, LambdaGCAndCluster CollectGCAndClustersResults, int32 NumDirty, int32 DispatchBatchIndex)
	{
		constexpr int32 MinRigidPerTask = 50;
		const int32 NumTasks = FMath::Max(FMath::Min(FTaskGraphInterface::Get().GetNumWorkerThreads() - 1, Chaos::MaxNumWorkers), 1);
		const int32 NumDirtyTask = FMath::Max(FMath::Min(NumTasks, NumDirty), 1);
		const int32 DirtyByTask = FMath::Max(FMath::DivideAndRoundUp(NumDirty, NumDirtyTask), MinRigidPerTask);
		const int32 LocalNumRigidBatches = FMath::DivideAndRoundUp(NumDirty, DirtyByTask);

		// We can have at most NumDirty overall active particles across all dirty proxy lists
		// and in each list at most NumProxies. So we take the min, otherwise for example in a world
		// with a few million static particles and one dynamic we'll allocate enough storage for the
		// millions of particles despite only ever syncing one
		for (int32 BatchIndex = 0; BatchIndex < LocalNumRigidBatches; BatchIndex++)
		{
			const int32 StartIndex = BatchIndex * DirtyByTask;
			int32 EndIndex = (BatchIndex + 1) * DirtyByTask;
			EndIndex = FMath::Min(NumDirty, EndIndex);

			TArray<FSingleParticlePhysicsProxy*>& ActiveRigid = ActiveRigidArray[DispatchBatchIndex + BatchIndex];
			ActiveRigid.Reset();

			// Create Task here 
			UE::Tasks::FTask DirtyPendingTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [StartIndex, EndIndex, &ActiveRigid, CollectRigidResults]()
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_CollectRigidsTasks);
				CollectRigidResults(StartIndex, EndIndex, ActiveRigid);
			});
			DirtyRigidPendingTasks.Add(DirtyPendingTask);
		}

		// The GC and cluster proxies are shared among several particles
		// all those following task will be computed sequentially to be sure not to add twice the same proxy. 
		// A fill all then merge approach could be potentially more efficient (not sure)
		UE::Tasks::FTask DirtyPendingTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, NumDirty, CollectGCAndClustersResults]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_CollectGCAndClustersTasks);
			CollectGCAndClustersResults(0, NumDirty, ActiveGC, ActiveClusterUnions);
		}, DirtyGCAndClusterPendingTask);
		DirtyGCAndClusterPendingTask = DirtyPendingTask;

		return LocalNumRigidBatches;
	}

	void BufferRigidResults(FPullPhysicsData* PullData);
	void BufferGCResults(FPullPhysicsData* PullData, FPBDRigidsSolver* RigidsSolver);
	void BufferClusterResults(FPullPhysicsData* PullData);
	void WaitTaskEndBufferResults() const;
	
};
}
