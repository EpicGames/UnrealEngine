// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Misc/App.h"
#include "AutoRTFM.h"
#include "Framework/Threading.h"
#include "Templates/UniquePtr.h"

#if WITH_DEV_AUTOMATION_TESTS

#define CHECK_EQ(A, B) \
	do { TestEqual(TEXT(__FILE__ ":" UE_STRINGIZE(__LINE__) ": TestEqual(" #A ", " #B ")"), (A), (B)); } while (0)

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAutoRTFMChaosPhysSceneLock, "AutoRTFM + FPhysSceneLock", \
	                             EAutomationTestFlags::EngineFilter | EAutomationTestFlags::ClientContext | \
								 EAutomationTestFlags::ServerContext | EAutomationTestFlags::CommandletContext)

bool FAutoRTFMChaosPhysSceneLock::RunTest(const FString& Parameters)
{
	if (!AutoRTFM::ForTheRuntime::IsAutoRTFMRuntimeEnabled())
	{
		ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Info, 
								                TEXT("SKIPPED 'FAutoRTFMChaosPhysSceneLock' test. AutoRTFM disabled.")));
		return true;
	}

	// It should be safe to declare when not used.
	AutoRTFM::Transact([&]
		{
			Chaos::FPhysSceneLock CreateInTransaction;
		});

	// Read lock then abort is fine.
	AutoRTFM::Transact([&]
		{
			Chaos::FPhysSceneLock CreateInTransaction;
			CreateInTransaction.ReadLock();
			AutoRTFM::AbortTransaction();
		});

	// Read lock/unlock pair then abort is fine.
	AutoRTFM::Transact([&]
		{
			Chaos::FPhysSceneLock CreateInTransaction;

			CreateInTransaction.ReadLock();
			CreateInTransaction.ReadUnlock();
			AutoRTFM::AbortTransaction();
		});

	// Read lock/unlock pair fine.
	AutoRTFM::Transact([&]
		{
			Chaos::FPhysSceneLock CreateInTransaction;

			CreateInTransaction.ReadLock();
			CreateInTransaction.ReadUnlock();
		});

	// Write lock then abort is fine.
	AutoRTFM::Transact([&]
		{
			Chaos::FPhysSceneLock CreateInTransaction;
			CreateInTransaction.WriteLock();
			AutoRTFM::AbortTransaction();
		});

	// Write lock/unlock pair then abort is fine.
	AutoRTFM::Transact([&]
		{
			Chaos::FPhysSceneLock CreateInTransaction;

			CreateInTransaction.WriteLock();
			CreateInTransaction.WriteUnlock();
			AutoRTFM::AbortTransaction();
		});

	// Write lock/unlock pair fine.
	AutoRTFM::Transact([&]
		{
			Chaos::FPhysSceneLock CreateInTransaction;

			CreateInTransaction.WriteLock();
			CreateInTransaction.WriteUnlock();
		});

	// Now do the same but outwith a transaction.
	Chaos::FPhysSceneLock CreateOutwithTransaction;

	// Read lock then abort is fine.

	AutoRTFM::Transact([&]
		{
			CreateOutwithTransaction.ReadLock();
			AutoRTFM::AbortTransaction();
		});

	// Read lock/unlock pair then abort is fine.
	AutoRTFM::Transact([&]
		{
			CreateOutwithTransaction.ReadLock();
			CreateOutwithTransaction.ReadUnlock();
			AutoRTFM::AbortTransaction();
		});

	// Read lock/unlock pair fine.
	AutoRTFM::Transact([&]
		{
			CreateOutwithTransaction.ReadLock();
			CreateOutwithTransaction.ReadUnlock();
		});

	// Write lock then abort is fine.
	AutoRTFM::Transact([&]
		{
			CreateOutwithTransaction.WriteLock();
			AutoRTFM::AbortTransaction();
		});

	// Write lock/unlock pair then abort is fine.
	AutoRTFM::Transact([&]
		{
			CreateOutwithTransaction.WriteLock();
			CreateOutwithTransaction.WriteUnlock();
			AutoRTFM::AbortTransaction();
		});

	AutoRTFM::Transact([&]
		{
			CreateOutwithTransaction.WriteLock();
			CreateOutwithTransaction.WriteUnlock();
		});

	return true;
}

#endif  // WITH_DEV_AUTOMATION_TESTS
