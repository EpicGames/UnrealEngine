// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Misc/App.h"
#include "AutoRTFM.h"
#include "Chaos/RefCountedObject.h"
#include "Templates/UniquePtr.h"

#if WITH_DEV_AUTOMATION_TESTS

#define CHECK_EQ(A, B) \
	do { TestEqual(TEXT(__FILE__ ":" UE_STRINGIZE(__LINE__) ": TestEqual(" #A ", " #B ")"), (A), (B)); } while (0)

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAutoRTFMChaosRefCountedObject, "AutoRTFM + ChaosRefCountedObject", \
	                             EAutomationTestFlags::EngineFilter | EAutomationTestFlags::ClientContext | \
								 EAutomationTestFlags::ServerContext | EAutomationTestFlags::CommandletContext)

bool FAutoRTFMChaosRefCountedObject::RunTest(const FString& Parameters)
{
	if (!AutoRTFM::ForTheRuntime::IsAutoRTFMRuntimeEnabled())
	{
		ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Info, 
								                TEXT("SKIPPED 'FAutoRTFMChaosRefCountedObject' test. AutoRTFM disabled.")));
		return true;
	}

	// It should be safe to declare a ref-counted object which is not used.
	AutoRTFM::Transact([&]
		{
			Chaos::FChaosRefCountedObject DoNothingObject;
		});

	// Adding and releasing a reference to a transient object will cause it to delete itself.
	{
		auto* TransientObject = new Chaos::FChaosRefCountedObject();
		AutoRTFM::Transact([&]
			{
				TransientObject->AddRef();
				TransientObject->Release();
			});
	}

	// AddRef on an object with non-zero refcount is rolled back properly when a transaction is aborted.
	{
		auto* TransientObject = new Chaos::FChaosRefCountedObject();
		TransientObject->AddRef();
		CHECK_EQ(TransientObject->GetRefCount(), 1);
		AutoRTFM::Transact([&]
			{
				TransientObject->AddRef();
				CHECK_EQ(TransientObject->GetRefCount(), 2);
				AutoRTFM::AbortTransaction();
			});
		CHECK_EQ(TransientObject->GetRefCount(), 1);
		TransientObject->Release();
	}

	// Release is rolled back properly when a transaction is aborted.
	{
		auto* TransientObject = new Chaos::FChaosRefCountedObject();
		TransientObject->AddRef();
		CHECK_EQ(TransientObject->GetRefCount(), 1);
		AutoRTFM::Transact([&]
			{
				TransientObject->Release();
				AutoRTFM::AbortTransaction();
			});
		CHECK_EQ(TransientObject->GetRefCount(), 1);
		TransientObject->Release();
	}

	// AddRef on a zero-refcount object is rolled back properly when a transaction is aborted.
	// (That is, the refcount is restored and the object is not destroyed.)
	{
		auto TransientObject = MakeUnique<Chaos::FChaosRefCountedObject>();
		AutoRTFM::Transact([&]
			{
				TransientObject->AddRef();
				AutoRTFM::AbortTransaction();
			});
		CHECK_EQ(TransientObject->GetRefCount(), 0);
	}

	// Adding and releasing a reference to a persistent object will not delete it.
	// This tests uses TUniquePtr to perform the deletion when the object falls out of scope.
	{
		auto PersistentObject = MakeUnique<Chaos::FChaosRefCountedObject>();
		PersistentObject->MakePersistent();
		AutoRTFM::Transact([&]
			{
				PersistentObject->AddRef();
				PersistentObject->Release();
			});
	}

	// It is safe to make an object persistent inside of a transaction.
	{
		auto PersistentObject = MakeUnique<Chaos::FChaosRefCountedObject>();
		AutoRTFM::Transact([&]
			{
				PersistentObject->MakePersistent();
				PersistentObject->AddRef();
				PersistentObject->Release();
			});
	}

	return true;
}

#undef CHECK_EQ

#endif  // WITH_DEV_AUTOMATION_TESTS
