// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "PhysicsProxy/AnalyticImplicitGroup.h"

CHAOS_API DECLARE_LOG_CATEGORY_EXTERN(USkeletalMeshSimulationComponentLogging, NoLogging, All);
//CHAOS_API DECLARE_LOG_CATEGORY_EXTERN(USkeletalMeshSimulationComponentLogging, Log, All);

/**
 * A hierarchy of transforms (bones) that tracks the state of transforms within 
 * the hierarchy, to do partial updates.
 */
// @todo(ccaulfield): we should not have to create bodies/aggregates for all parents of a simulated bone
// @todo(ccaulfield): fix dirty flag propagation and support setting of component-space poses and initialization from anim fo local and component poses
// @todo(ccaulfield): should have better separation of mapping from anim to physics and physics i/o data
// @todo(ccaulfield): actually I think this class only needs local-space transforms for non-physics child bones (or interstitials). The rest just need "simulation space"
class FBoneHierarchy
{
public:
	FBoneHierarchy()
		: ActorLocalToWorldDirty(false)
	{}
	FBoneHierarchy(const FBoneHierarchy &) = delete;
	FBoneHierarchy(FBoneHierarchy&& Other) { *this = MoveTemp(Other); }

	FBoneHierarchy& operator=(const FBoneHierarchy&) = delete;
	FBoneHierarchy& operator=(FBoneHierarchy&& Other)
	{
		ImplicitGroups = MoveTemp(Other.ImplicitGroups);
		BoneIndices = MoveTemp(Other.BoneIndices);
		SocketIndices = MoveTemp(Other.SocketIndices);

		BoneToShapeGroup = MoveTemp(Other.BoneToShapeGroup);
		Roots = MoveTemp(Other.Roots);

		BoneToTransformIndex = MoveTemp(Other.BoneToTransformIndex);
		TransformToBoneIndex = MoveTemp(Other.TransformToBoneIndex);

		LocalSpaceTransforms = MoveTemp(Other.LocalSpaceTransforms);
		WorldSpaceTransforms = MoveTemp(Other.WorldSpaceTransforms);
		PrevWorldSpaceTransforms = MoveTemp(Other.PrevWorldSpaceTransforms);

		AnimDirty = MoveTemp(Other.AnimDirty);

		ParentIndices = MoveTemp(Other.ParentIndices);
		ChildIndices = MoveTemp(Other.ChildIndices);

		ActorLocalToWorld = MoveTemp(Other.ActorLocalToWorld);
		ActorLocalToWorldDirty = MoveTemp(Other.ActorLocalToWorldDirty);

		TempTargetIndices = MoveTemp(Other.TempTargetIndices);

		return *this;
	}

	//--------------------------------------------------------------------------
	// Initialization
	//--------------------------------------------------------------------------

	/**
	 * Prepare the hierarchy for adding structures (optional).
	 */
	void InitPreAdd(const int32 NumBones)
	{
		ImplicitGroups.Reset();
		ImplicitGroups.Reserve(NumBones);
		BoneIndices.Reset();
		BoneIndices.Reserve(NumBones);
		BoneToShapeGroup.Reset();
		BoneToShapeGroup.Reserve(NumBones);
	}

	/**
	 * Add a structure to the hierarchy.
	 */
	void Add(TUniquePtr<FAnalyticImplicitGroup> &&AnalyticShapeGroupIn)
	{
		// Make sure the shape group is valid (has a bone id).
		check(AnalyticShapeGroupIn->IsValid());

		const int32 Index = ImplicitGroups.Add(MoveTemp(AnalyticShapeGroupIn));
		const TUniquePtr<FAnalyticImplicitGroup> &AnalyticShapeGroup = ImplicitGroups[Index];

		check(BoneIndices.Find(AnalyticShapeGroup->GetBoneIndex()) == INDEX_NONE);
		BoneIndices.Add(AnalyticShapeGroup->GetBoneIndex());
		check(BoneToShapeGroup.Contains(AnalyticShapeGroup->GetBoneIndex()) == false);
		BoneToShapeGroup.Add(AnalyticShapeGroup->GetBoneIndex(), AnalyticShapeGroup.Get());
		check(BoneToShapeGroup.Num() == BoneIndices.Num()); // Duplicate bone index or hash collision?
	}

	/**
	 * Build the hierarchy.  Must be done after adding structures, and before 
	 * updating or sampling.
	 */
	void InitPostAdd()
	{ 
		InitHierarchy(); 
	}

	bool HasBoneIndex(const uint32 BoneIndex) const
	{
		return BoneToShapeGroup.Contains(BoneIndex);
	}

	const TArray<int32>& GetBoneIndices() const
	{
		return BoneIndices;
	}

	void SetSocketIndexForBone(const uint32 BoneIndex, const int32 SocketIndex)
	{
		SocketIndices[BoneToTransformIndex[BoneIndex]] = SocketIndex;
	}

	int32 GetSocketIndexForBone(const uint32 BoneIndex) const
	{
		return SocketIndices[BoneToTransformIndex[BoneIndex]];
	}

	const TArray<TUniquePtr<FAnalyticImplicitGroup>>& GetAnalyticShapeGroups() const
	{
		return ImplicitGroups;
	}

	TArray<TUniquePtr<FAnalyticImplicitGroup>>& GetAnalyticShapeGroups()
	{
		return ImplicitGroups;
	}

	FAnalyticImplicitGroup* GetAnalyticShapeGroup(const int32 BoneIndex)
	{
		return BoneToShapeGroup[BoneIndex];
	}

	const FAnalyticImplicitGroup* GetAnalyticShapeGroup(const int32 BoneIndex) const
	{
		return BoneToShapeGroup[BoneIndex];
	}

public:
	//--------------------------------------------------------------------------
	// Anim update interface
	//--------------------------------------------------------------------------

	void PrepareForUpdate()
	{
		for (int32 Index = 0; Index < WorldSpaceTransforms.Num(); Index++)
		{
			PrevWorldSpaceTransforms[Index] = WorldSpaceTransforms[Index];
		}
	}

	/**
	 * Updates the local/bone space transform of a bone.
	 *
	 * Setting transforms at this level requires the transform hierarchy to be 
	 * resolved prior to sampling.
	 */
	void SetAnimLocalSpaceTransform(const int32 BoneIndex, const FTransform &BoneXf)
	{
		check(BoneToTransformIndex.Contains(BoneIndex));
		const int32 TransformIndex = BoneToTransformIndex[BoneIndex];
		FTransform& TargetXf = LocalSpaceTransforms[TransformIndex];
		// FTransform::Equals() is flakey.
		//if (!TargetXf.Equals(BoneXf, 1.0e-8))
		if (!TargetXf.GetTranslation().Equals(BoneXf.GetTranslation(), 1.0e-8f) ||
			!TargetXf.GetRotation().Equals(BoneXf.GetRotation(), 1.0e-8f) ||
			!TargetXf.GetScale3D().Equals(BoneXf.GetScale3D(), 1.0e-8f))
		{
			TargetXf = BoneXf;
			SetAnimLocalDirty(TransformIndex);
			check(GetAnimLocalDirty(TransformIndex) == true);
		}
	}

	/**
	 * Update the local-to-world transform of the actor the bones this hierarchy 
	 * represents belongs to.
	 */
	void SetActorWorldSpaceTransform(const FTransform &InActorLocalToWorld)
	{
		// FTransform::Equals() is flakey.
		//ActorLocalToWorldDirty |= !ActorLocalToWorld.FTransform::Equals(InActorLocalToWorld);
		ActorLocalToWorldDirty |=
			!ActorLocalToWorld.GetTranslation().Equals(InActorLocalToWorld.GetTranslation(), 1.0e-8f) ||
			!ActorLocalToWorld.GetRotation().Equals(InActorLocalToWorld.GetRotation(), 1.0e-8f) ||
			!ActorLocalToWorld.GetScale3D().Equals(InActorLocalToWorld.GetScale3D(), 1.0e-8f);
		ActorLocalToWorld = InActorLocalToWorld;
	}

public:
	//--------------------------------------------------------------------------
	// Anim query interface
	//--------------------------------------------------------------------------

	/**
	 * Update all world space transforms for sampling.
	 */
	bool PrepareAnimWorldSpaceTransforms()
	{
		PropagateAnimDirtyFlags();
		GetDirtyAnimIndices(TempTargetIndices);
		bool DidSomething = UpdateAnimWorldSpaceTransforms(TempTargetIndices);
		check(ActorLocalToWorldDirty == false);
		return DidSomething;
	}

	/**
	 * Get all world space transforms associated with \p BoneIndex.  
	 *
	 * The transform for the bone is offset index 0 of the return value.  If 
	 * this bone has associated sub structures, and the hierarchy was initialized 
	 * with \c FirstClassSubStructures = \c true, they start at index 1.
	 */
	const FTransform* GetAnimWorldSpaceTransformsForBone(const int32 BoneIndex) const
	{
		// Make sure this bone is in the hierarchy.
		check(BoneToTransformIndex.Contains(BoneIndex));
		// Make sure PrepareWorldSpaceTransforms() has been called.
		check(ActorLocalToWorldDirty == false); 
		return &WorldSpaceTransforms[BoneToTransformIndex[BoneIndex]];
	}

	const FTransform* GetPrevAnimWorldSpaceTransformForBone(const int32 BoneIndex) const
	{
		const int32 TransformIndex = BoneToTransformIndex[BoneIndex];
		return PrevWorldSpaceTransforms.IsValidIndex(TransformIndex) ? &PrevWorldSpaceTransforms[TransformIndex] : nullptr;
	}

	int32 GetTransformIndex(const int32 BoneIndex) const
	{
		return BoneToTransformIndex[BoneIndex];
	}

public:
	//--------------------------------------------------------------------------
	// Sim interface
	//--------------------------------------------------------------------------


protected:
	/**
	 * Update each implicit group with parent and child information, and find root nodes.
	 *
	 * Bones are sorted so that those with kinematic or dynamic bodies are first, and
	 * sorted by index (this allows to to iterate over the bone list when producing
	 * output for the animation system which asserts that the bone update list is sorted
	 * as an overly-sufficient check that parents are before children).
	 */
	void InitHierarchy()
	{
		// Sort the bone Indices. Bones with physics (kinematic or dynamic) first, then by Index
		BoneIndices.Sort(
			[this](const int32 BoneIndexL, const int32 BoneIndexR)
			{
				const FAnalyticImplicitGroup* GroupL = GetAnalyticShapeGroup(BoneIndexL);
				const FAnalyticImplicitGroup* GroupR = GetAnalyticShapeGroup(BoneIndexR);
				const bool bHasBodyL = GroupL->NumStructures() > 0;
				const bool bHasBodyR = GroupR->NumStructures() > 0;

				if (bHasBodyL && !bHasBodyR)
				{
					return true;
				}
				else if (!bHasBodyL && bHasBodyR)
				{
					return false;
				}

				return BoneIndexL < BoneIndexR;
			});

		for (TUniquePtr<FAnalyticImplicitGroup> &Group : ImplicitGroups)
		{
			// Remove parents and children, but not parent bone index.
			Group->ClearHierarchy();
		}
		Roots.Reset();
		int32 NumTransforms = 0;
		for (TUniquePtr<FAnalyticImplicitGroup> &Group : ImplicitGroups)
		{
			const int32 ParentBoneIndex = Group->GetParentBoneIndex();
			if (ParentBoneIndex == INDEX_NONE)
			{
				Roots.Add(Group.Get());
			}
			else
			{
				check(BoneToShapeGroup.Contains(ParentBoneIndex));
				FAnalyticImplicitGroup *Parent = BoneToShapeGroup[ParentBoneIndex];
				check(Parent != nullptr);
				Parent->AddChild(Group.Get());
				Group->SetParent(Parent);
				check(Group->GetParent() == Parent);
			}

			// Count the number of transforms we have.
			NumTransforms++;
		}
		check(Roots.Num() != 0);
		check(NumTransforms == ImplicitGroups.Num());
		UE_LOGF(USkeletalMeshSimulationComponentLogging, Log,
			"USkeletalMeshPhysicsProxy::InitHierarchy() - this: %p - "
				"Implicit groups: %d, num transforms: %d, num roots: %d",
			this,
			ImplicitGroups.Num(),
			NumTransforms,
			Roots.Num());

		// Initialize transform buffers
		LocalSpaceTransforms.Init(FTransform::Identity, NumTransforms);
		WorldSpaceTransforms.Init(FTransform::Identity, NumTransforms);
		PrevWorldSpaceTransforms.Init(FTransform::Identity, NumTransforms);
		AnimDirty.Init(0, NumTransforms);

		SocketIndices.Init(INDEX_NONE, NumTransforms);
		ParentIndices.Init(INDEX_NONE, NumTransforms);
		ChildIndices.Reset();
		ChildIndices.AddDefaulted(NumTransforms);

		// Iterate over all nodes in hierarchical order, starting with the root(s).  
		// As we go, we append the children to our traversal set.
		TArray<const TArray<FAnalyticImplicitGroup*>*> TraversalGroups;
		TraversalGroups.Reserve(ImplicitGroups.Num());
		TraversalGroups.Add(&Roots);
		int32 TransformIndex = 0;
		for (int32 i = 0; i < TraversalGroups.Num(); i++)
		{
			const TArray<FAnalyticImplicitGroup*> *Groups = TraversalGroups[i];
			for (FAnalyticImplicitGroup* Group : *Groups)
			{
				const int32 BoneIndex = Group->GetBoneIndex();
				BoneToTransformIndex.Add(BoneIndex, TransformIndex);
				TransformToBoneIndex.Add(TransformIndex, BoneIndex);

				// Add this group to its parents list of children.
				if (const FAnalyticImplicitGroup* ParentGroup = Group->GetParent())
				{
					const int32 ParentBoneIndex = ParentGroup->GetBoneIndex();
					check(ParentBoneIndex != INDEX_NONE);
					check(BoneToTransformIndex.Contains(ParentBoneIndex));
					const int32 ParentTransformIndex = BoneToTransformIndex[ParentBoneIndex];
					ChildIndices[ParentTransformIndex].Add(TransformIndex);
					ParentIndices[TransformIndex] = ParentTransformIndex;
				}
				else
				{
					check(Group->GetParentBoneIndex() == INDEX_NONE);
					check(Roots.Find(Group) != INDEX_NONE);
				}

				const TArray<FAnalyticImplicitGroup*>& Children = Group->GetChildren();
				ChildIndices[TransformIndex].Reserve(/*Group->NumStructures() + */Children.Num());
				TransformIndex++;

				// The children haven't been assigned a transform index yet, so 
				// all we can do is allocate memory.  We'll populate the ChildIndices
				// lists with children when we iterate over them.

				if (Children.Num())
				{
					// Add the children of this bone to the traversal set.
					TraversalGroups.Add(&Children);
				}

			}
		}
	}

protected:
	/**
	 * Update the component space transforms of the specified indices.
	 */
	bool UpdateAnimWorldSpaceTransforms(const TArray<int32>& TargetIndices)
	{
		//           * R
		//           |
		//     A *-------* B
		//       |       |
		//       |       |
		//     C *       * D
		//       |
		// E *-------* F
		//   |
		//   |
		// G *
		//
		// World space G = T(RA) * T(AC) * T(CE) * T(EG)

		for(const int32 TransformIndex : TargetIndices)
		{
			check(GetAnimLocalDirty(TransformIndex) == true);

			const int32 ParentTransformIndex = ParentIndices[TransformIndex];
			const FTransform& ChildXf = LocalSpaceTransforms[TransformIndex];
			FTransform& WorldParentToChildXf = WorldSpaceTransforms[TransformIndex];
			if (ParentTransformIndex == INDEX_NONE)
			{
				// This is a root in our hierarchy.  Transform by the actor's Xf.
				// T(RA)
				WorldParentToChildXf = ChildXf * ActorLocalToWorld;
			}
			else
			{
				// This group has a parent.  Multiply by the parent transform.
				// T(AC)
				check(GetAnimLocalDirty(ParentTransformIndex) == false);
				const FTransform& ParentXf = WorldSpaceTransforms[ParentTransformIndex];
				WorldParentToChildXf = ChildXf * ParentXf;
			}

			SetAnimLocalClean(TransformIndex);
			check(GetAnimLocalDirty(TransformIndex) == false);
		}
		ActorLocalToWorldDirty = false;
		return (TargetIndices.Num() > 0);
	}

	/**
	 * Propagate bone space dirty flags from parents to sub structures and children.
	 */
	void PropagateAnimDirtyFlags()
	{
		if (ActorLocalToWorldDirty)
		{
			for (int32 TransformIndex = 0; TransformIndex < AnimDirty.Num(); TransformIndex++)
				SetAnimLocalDirty(TransformIndex);
		}
		else
		{
			for (int32 TransformIndex = 0; TransformIndex < ChildIndices.Num(); TransformIndex++)
				if (GetAnimLocalDirty(TransformIndex))
				{
					for (const int32 ChildTransformIndex : ChildIndices[TransformIndex])
						SetAnimLocalDirty(ChildTransformIndex);
				}
		}
	}

	/**
	 * Get a list of all transform indices that are dirty.
	 */
	void GetDirtyIndices(TArray<int32>& TargetIndices, const uint8 Mask) const
	{
		// Collect indices that need an update, in order.
		TargetIndices.Reset();
		for (int i = 0; i < AnimDirty.Num(); i++)
		{
			if (AnimDirty[i] & Mask)
				TargetIndices.Add(i);
		}
	}

	// @todo(ccaulfield): remove hard-coded bit fields
	// @todo(ccaulfield): this needs to separate local and component-space dirty flags.
	// @todo(ccaulfield): Changing a local-space pose should dirty the Component-space children
	// @todo(ccaulfield): Changing a component-space pose should dirty the local-space pose (which will dirty child component space poses)
	void GetDirtyAnimIndices(TArray<int32>& TargetIndices)
	{ GetDirtyIndices(TargetIndices, 0b0001); }

	bool GetAnimLocalDirty(const int32 TransformIndex) const
	{ return AnimDirty[TransformIndex] & 0b0001; }
	void SetAnimLocalDirty(const int32 TransformIndex)
	{ AnimDirty[TransformIndex] |= 0b0001; }
	void SetAnimLocalClean(const int32 TransformIndex)
	{ AnimDirty[TransformIndex] &= 0b1110; }
protected:
	//friend class USkeletalMeshSimulationComponent;

	// Owner of all implicit shape groups.
	TArray<TUniquePtr<FAnalyticImplicitGroup> > ImplicitGroups;
	TArray<int32> BoneIndices;
	TArray<int32> SocketIndices;

	// A mapping from Bone index to implicit shape group.
	TMap<int32, FAnalyticImplicitGroup*> BoneToShapeGroup;
	// All implicit groups in the hierarchy that have no parents.
	TArray<FAnalyticImplicitGroup*> Roots;

	// Mapping from Bone index to our local transform array indices.
	TMap<int32, int32> BoneToTransformIndex;
	TMap<int32, int32> TransformToBoneIndex;

	// Local, component, and world space transforms of each bone in the hierarchy, 
	// plus sub group structures.
	TArray<FTransform> LocalSpaceTransforms;
	//TArray<FTransform> ComponentSpaceTransforms;
	TArray<FTransform> WorldSpaceTransforms;
	TArray<FTransform> PrevWorldSpaceTransforms;
	// Dirty flags for each bone in the hierarchy, plus sub group structures.
	TArray<uint8> AnimDirty;

	// Parenting hierarchy for each bone and sub group structure
	TArray<int32> ParentIndices;
	TArray<TArray<int32>> ChildIndices;

	// The current top level local-to-world transform.
	FTransform ActorLocalToWorld;
	bool ActorLocalToWorldDirty;

	// The working set of indices.
	TArray<int32> TempTargetIndices;
};
