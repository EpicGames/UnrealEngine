// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/AABB.h"
#include "Chaos/AABBVectorized.h"
#include "Chaos/AABBVectorizedDouble.h"
#include "Chaos/AABBTreeDirtyGridUtils.h"
#include "Chaos/Defines.h"
#include "Chaos/GeometryParticles.h"
#include "Chaos/Transform.h"
#include "ChaosLog.h"
#include "Chaos/ISpatialAcceleration.h"
#include "Templates/Models.h"
#include "Chaos/BoundingVolume.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "ChaosStats.h"
#include "Math/VectorRegister.h"
#include <type_traits>

#ifndef CHAOS_DEBUG_NAME
#define CHAOS_DEBUG_NAME 0
#endif

CSV_DECLARE_CATEGORY_EXTERN(ChaosPhysicsTimers);

struct FAABBTreeCVars
{
	static CHAOS_API int32 UpdateDirtyElementPayloadData;
	static CHAOS_API FAutoConsoleVariableRef CVarUpdateDirtyElementPayloadData;

	static CHAOS_API int32 SplitAtAverageCenter;
	static CHAOS_API FAutoConsoleVariableRef CVarSplitAtAverageCenter;

	static CHAOS_API int32 SplitOnVarianceAxis;
	static CHAOS_API FAutoConsoleVariableRef CVarSplitOnVarianceAxis;

	static CHAOS_API float DynamicTreeBoundingBoxPadding;
	static CHAOS_API FAutoConsoleVariableRef CVarDynamicTreeBoundingBoxPadding;

	static CHAOS_API float DynamicTreeLeafEnlargePercent;
	static CHAOS_API FAutoConsoleVariableRef CVarDynamicTreeLeafEnlargePercent;

	static CHAOS_API int32 DynamicTreeLeafCapacity;
	static CHAOS_API FAutoConsoleVariableRef CVarDynamicTreeLeafCapacity;

};

struct FAABBTreeDirtyGridCVars
{
	static CHAOS_API int32 DirtyElementGridCellSize;
	static CHAOS_API FAutoConsoleVariableRef CVarDirtyElementGridCellSize;

	static CHAOS_API int32 DirtyElementMaxGridCellQueryCount;
	static CHAOS_API FAutoConsoleVariableRef CVarDirtyElementMaxGridCellQueryCount;

	static CHAOS_API int32 DirtyElementMaxPhysicalSizeInCells;
	static CHAOS_API FAutoConsoleVariableRef CVarDirtyElementMaxPhysicalSizeInCells;

	static CHAOS_API int32 DirtyElementMaxCellCapacity;
	static CHAOS_API FAutoConsoleVariableRef CVarDirtyElementMaxCellCapacity;
};

struct FAABBTimeSliceCVars
{
	static CHAOS_API bool bUseTimeSliceMillisecondBudget;
	static CHAOS_API FAutoConsoleVariableRef CVarUseTimeSliceByMillisecondBudget;

	static CHAOS_API float MaxProcessingTimePerSliceSeconds;
	static CHAOS_API FAutoConsoleVariableRef CVarMaxProcessingTimePerSlice;

	static CHAOS_API int32 MinNodesChunkToProcessBetweenTimeChecks;
	static CHAOS_API FAutoConsoleVariableRef CVarMinNodesChunkToProcessBetweenTimeChecks;

	static CHAOS_API int32 MinDataChunkToProcessBetweenTimeChecks;
	static CHAOS_API FAutoConsoleVariableRef CVarMinDataChunkToProcessBetweenTimeChecks;
};

class FChaosVDDataWrapperUtils;

namespace Chaos
{

enum class EAABBQueryType
{
	Raycast,
	Sweep,
	Overlap
};

struct AABBTreeStatistics
{
	void Reset()
	{
		StatNumNonEmptyCellsInGrid = 0;
		StatNumElementsTooLargeForGrid = 0;
		StatNumDirtyElements = 0;
		StatNumGridOverflowElements = 0;
	}

	AABBTreeStatistics& MergeStatistics(const AABBTreeStatistics& Rhs)
	{
		StatNumNonEmptyCellsInGrid += Rhs.StatNumNonEmptyCellsInGrid;
		StatNumElementsTooLargeForGrid += Rhs.StatNumElementsTooLargeForGrid;
		StatNumDirtyElements += Rhs.StatNumDirtyElements;
		StatNumGridOverflowElements += Rhs.StatNumGridOverflowElements;
		return *this;
	}

	int32 StatNumNonEmptyCellsInGrid = 0;
	int32 StatNumElementsTooLargeForGrid = 0;
	int32 StatNumDirtyElements = 0;
	int32 StatNumGridOverflowElements = 0;
};

struct AABBTreeExpensiveStatistics
{
	void Reset()
	{
		StatMaxNumLeaves = 0;
		StatMaxDirtyElements = 0;
		StatMaxLeafSize = 0;
		StatMaxTreeDepth = 0;
		StatGlobalPayloadsSize = 0;
	}

	AABBTreeExpensiveStatistics& MergeStatistics(const AABBTreeExpensiveStatistics& Rhs)
	{
		StatMaxNumLeaves = FMath::Max(StatMaxNumLeaves, Rhs.StatMaxNumLeaves);
		StatMaxDirtyElements = FMath::Max(StatMaxDirtyElements, Rhs.StatMaxDirtyElements);
		StatMaxLeafSize = FMath::Max(StatMaxLeafSize, Rhs.StatMaxLeafSize);
		StatMaxTreeDepth = FMath::Max(StatMaxTreeDepth, Rhs.StatMaxTreeDepth);
		StatGlobalPayloadsSize += Rhs.StatGlobalPayloadsSize;
		return *this;
	}
	int32 StatMaxNumLeaves = 0;
	int32 StatMaxDirtyElements = 0;
	int32 StatMaxLeafSize = 0;
	int32 StatMaxTreeDepth = 0;
	int32 StatGlobalPayloadsSize = 0;
};

DECLARE_CYCLE_STAT(TEXT("AABBTreeGenerateTree"), STAT_AABBTreeGenerateTree, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("AABBTreeTimeSliceSetup"), STAT_AABBTreeTimeSliceSetup, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("AABBTreeInitialTimeSlice"), STAT_AABBTreeInitialTimeSlice, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("AABBTreeProgressTimeSlice"), STAT_AABBTreeProgressTimeSlice, STATGROUP_Chaos);
//DECLARE_CYCLE_STAT(TEXT("AABBTreeGrowPhase"), STAT_AABBTreeGrowPhase, STATGROUP_Chaos);
//DECLARE_CYCLE_STAT(TEXT("AABBTreeChildrenPhase"), STAT_AABBTreeChildrenPhase, STATGROUP_Chaos);

struct CIsUpdatableElement
{
	template<typename ElementT>
	auto Requires(ElementT& InElem, const ElementT& InOtherElem) -> decltype(InElem.UpdateFrom(InOtherElem));
};

template<typename T, typename TEnableIf<!TModels_V<CIsUpdatableElement, T>>::Type* = nullptr>
static void UpdateElementHelper(T& InElem, const T& InFrom)
{

}

template<typename T, typename TEnableIf<TModels_V<CIsUpdatableElement, T>>::Type* = nullptr>
static void UpdateElementHelper(T& InElem, const T& InFrom)
{
	if (FAABBTreeCVars::UpdateDirtyElementPayloadData != 0)
	{
		InElem.UpdateFrom(InFrom);
	}
}

template <typename TQueryFastData, EAABBQueryType Query>
struct TAABBTreeIntersectionHelper
{
	static bool Intersects(const FVec3& Start, TQueryFastData& QueryFastData, FReal& TOI,
		const FAABB3& Bounds, const FAABB3& QueryBounds, const FVec3& QueryHalfExtents, const FVec3& Dir, const FVec3 InvDir, const bool bParallel[3])
	{
		check(false);
		return true;
	}
};

template<>
struct TAABBTreeIntersectionHelper<FQueryFastData, EAABBQueryType::Raycast>
{
	FORCEINLINE_DEBUGGABLE static bool Intersects(const FVec3& Start, FQueryFastData& QueryFastData, FReal& TOI,
		const FAABB3& Bounds, const FAABB3& QueryBounds, const FVec3& QueryHalfExtents, const FVec3& Dir, const FVec3 InvDir, const bool bParallel[3])
	{
		FReal TmpExitTime;
		return Bounds.RaycastFast(Start, Dir, InvDir, bParallel, QueryFastData.CurrentLength, QueryFastData.InvCurrentLength, TOI, TmpExitTime);
	}

};

template <>
struct TAABBTreeIntersectionHelper<FQueryFastData, EAABBQueryType::Sweep>
{
	FORCEINLINE_DEBUGGABLE static bool Intersects(const FVec3& Start, FQueryFastData& QueryFastData, FReal& TOI,
		const FAABB3& Bounds, const FAABB3& QueryBounds, const FVec3& QueryHalfExtents, const FVec3& Dir, const FVec3 InvDir, const bool bParallel[3])
	{
		FAABB3 SweepBounds(Bounds.Min() - QueryHalfExtents, Bounds.Max() + QueryHalfExtents);
		FReal TmpExitTime;
		return SweepBounds.RaycastFast(Start, Dir, InvDir, bParallel, QueryFastData.CurrentLength, QueryFastData.InvCurrentLength, TOI, TmpExitTime);
	}

};

template <>
struct TAABBTreeIntersectionHelper<FQueryFastDataVoid, EAABBQueryType::Overlap>
{
	FORCEINLINE_DEBUGGABLE static bool Intersects(const FVec3& Start, FQueryFastDataVoid& QueryFastData, FReal& TOI,
		const FAABB3& Bounds, const FAABB3& QueryBounds, const FVec3& QueryHalfExtents, const FVec3& Dir, const FVec3 InvDir, const bool bParallel[3])
	{
		return QueryBounds.Intersects(Bounds);
	}
};

template <typename TPayloadType, typename T, bool bComputeBounds>
struct TBoundsWrapperHelper
{
};

template <typename TPayloadType, typename T>
struct TBoundsWrapperHelper<TPayloadType, T, true>
{
	void ComputeBounds(const TArray<TPayloadBoundsElement<TPayloadType, T>>& Elems, bool bDynamicTree)
	{
		Bounds = TAABB<T, 3>::EmptyAABB();

		for (const auto& Elem : Elems)
		{
			Bounds.GrowToInclude(Elem.Bounds);
		}
		if (!Elems.IsEmpty() && bDynamicTree)
		{
			// Grow the aabb leaf of percentage of its size
			Bounds.ThickenSymmetrically(Bounds.Extents() * FAABBTreeCVars::DynamicTreeLeafEnlargePercent*0.5);
		}
	}

	const TAABB<T, 3>& GetBounds() const { return Bounds; }

private:
	TAABB<T, 3> Bounds;
};

template <typename TPayloadType, typename T>
struct TBoundsWrapperHelper<TPayloadType, T, false>
{
	void ComputeBounds(const TArray<TPayloadBoundsElement<TPayloadType, T>>&, bool bDynamicTree)
	{
	}

	const TAABB<T, 3> GetBounds() const
	{
		return TAABB<T, 3>::EmptyAABB();
	}
};

template <typename TPayloadType, bool bComputeBounds = true, typename T = FReal>
struct TAABBTreeLeafArray : public TBoundsWrapperHelper<TPayloadType, T, bComputeBounds>
{
	TAABBTreeLeafArray() {}
	//todo: avoid copy?
	TAABBTreeLeafArray(const TArray<TPayloadBoundsElement<TPayloadType, T>>& InElems)
		: Elems(InElems)
	{
		this->ComputeBounds(Elems, false);
	}

	void GatherElements(TArray<TPayloadBoundsElement<TPayloadType, T>>& OutElements) const
	{
		OutElements.Append(Elems);
	}

	SIZE_T GetReserveCount() const
	{
		// Optimize for fewer memory allocations.
		return Elems.Num();
	}

	SIZE_T GetElementCount() const
	{
		return Elems.Num();
	}

	void RecomputeBounds(bool bDynamicTree)
	{
		this->ComputeBounds(Elems, bDynamicTree);
	}

	/** Check if the leaf is dirty (if one of the payload have been updated)
	 * @return Dirty boolean that indicates if the leaf is dirty or not
 	 */
	bool IsLeafDirty() const
	{
		return bDirtyLeaf;
	}

	/** Set thye dirty flag onto the leaf 
	 * @param  bDirtyState Disrty flag to set 
	 */
	void SetDirtyState(const bool bDirtyState)
	{
		bDirtyLeaf = bDirtyState;
	}

	template <typename TSQVisitor, typename TQueryFastData>
	FORCEINLINE_DEBUGGABLE bool RaycastFast(const TVec3<T>& Start, TQueryFastData& QueryFastData, TSQVisitor& Visitor, const TVec3<T>& Dir, const TVec3<T>& InvDir, const bool bParallel[3]) const
	{
		return RaycastSweepImp</*bSweep=*/false>(Start, QueryFastData, TVec3<T>((T)0), Visitor, Dir, InvDir, bParallel);
	}

	template <typename TSQVisitor, typename TQueryFastData>
	FORCEINLINE_DEBUGGABLE bool RaycastFastSimd(const VectorRegister4Double& Start, TQueryFastData& QueryFastData, TSQVisitor& Visitor, const VectorRegister4Double& Dir, const VectorRegister4Double& InvDir, const VectorRegister4Double& Parallel, const VectorRegister4Double& Length) const
	{
		return RaycastImpSimd(Start, QueryFastData, Visitor, InvDir, Parallel, Length);
	}

	template <typename TSQVisitor, typename TQueryFastData>
	FORCEINLINE_DEBUGGABLE bool SweepFast(const TVec3<T>& Start, TQueryFastData& QueryFastData, const TVec3<T>& QueryHalfExtents, TSQVisitor& Visitor,
			const TVec3<T>& Dir, const TVec3<T> InvDir, const bool bParallel[3]) const
	{
		return RaycastSweepImp</*bSweep=*/true>(Start, QueryFastData, QueryHalfExtents, Visitor, Dir, InvDir, bParallel);
	}

	template <typename TSQVisitor>
	bool OverlapFast(const FAABB3& QueryBounds, TSQVisitor& Visitor) const
	{
		PHYSICS_CSV_CUSTOM_VERY_EXPENSIVE(PhysicsCounters, MaxLeafSize, Elems.Num(), ECsvCustomStatOp::Max);

		const int32 NumElems = Elems.Num();
		const int32 SimdIters = NumElems / 4;

		for (int32 SimdIter = 0; SimdIter < SimdIters; ++SimdIter)
		{
			const int32 StartIndex = SimdIter * 4;
			VectorRegister4Double MinX = MakeVectorRegisterDouble(Elems[StartIndex].Bounds.Min()[0], Elems[StartIndex + 1].Bounds.Min()[0], Elems[StartIndex + 2].Bounds.Min()[0], Elems[StartIndex + 3].Bounds.Min()[0]);
			VectorRegister4Double MaxX = MakeVectorRegisterDouble(Elems[StartIndex].Bounds.Max()[0], Elems[StartIndex + 1].Bounds.Max()[0], Elems[StartIndex + 2].Bounds.Max()[0], Elems[StartIndex + 3].Bounds.Max()[0]);
			VectorRegister4Double MinY = MakeVectorRegisterDouble(Elems[StartIndex].Bounds.Min()[1], Elems[StartIndex + 1].Bounds.Min()[1], Elems[StartIndex + 2].Bounds.Min()[1], Elems[StartIndex + 3].Bounds.Min()[1]);
			VectorRegister4Double MaxY = MakeVectorRegisterDouble(Elems[StartIndex].Bounds.Max()[1], Elems[StartIndex + 1].Bounds.Max()[1], Elems[StartIndex + 2].Bounds.Max()[1], Elems[StartIndex + 3].Bounds.Max()[1]);
			VectorRegister4Double MinZ = MakeVectorRegisterDouble(Elems[StartIndex].Bounds.Min()[2], Elems[StartIndex + 1].Bounds.Min()[2], Elems[StartIndex + 2].Bounds.Min()[2], Elems[StartIndex + 3].Bounds.Min()[2]);
			VectorRegister4Double MaxZ = MakeVectorRegisterDouble(Elems[StartIndex].Bounds.Max()[2], Elems[StartIndex + 1].Bounds.Max()[2], Elems[StartIndex + 2].Bounds.Max()[2], Elems[StartIndex + 3].Bounds.Max()[2]);

			const VectorRegister4Double OtherMinX = VectorSetDouble1(QueryBounds.Min().X);
			const VectorRegister4Double OtherMinY = VectorSetDouble1(QueryBounds.Min().Y);
			const VectorRegister4Double OtherMinZ = VectorSetDouble1(QueryBounds.Min().Z);

			const VectorRegister4Double OtherMaxX = VectorSetDouble1(QueryBounds.Max().X);
			const VectorRegister4Double OtherMaxY = VectorSetDouble1(QueryBounds.Max().Y);
			const VectorRegister4Double OtherMaxZ = VectorSetDouble1(QueryBounds.Max().Z);

			VectorRegister4Double IsFalseX = VectorBitwiseOr(VectorCompareGT(MinX, OtherMaxX), VectorCompareGT(OtherMinX, MaxX));
			VectorRegister4Double IsFalseY = VectorBitwiseOr(VectorCompareGT(MinY, OtherMaxY), VectorCompareGT(OtherMinY, MaxY));
			VectorRegister4Double IsFalseZ = VectorBitwiseOr(VectorCompareGT(MinZ, OtherMaxZ), VectorCompareGT(OtherMinZ, MaxZ));

			VectorRegister4Double IsFalse = VectorBitwiseOr(VectorBitwiseOr(IsFalseX, IsFalseY), IsFalseZ);

			int32 MaskBitFalse = VectorMaskBits(IsFalse);

			for (int32 MaskIndex = 0; MaskIndex < 4; ++MaskIndex)
			{
				if ((MaskBitFalse & (1 << MaskIndex)) == 0)
				{
					const TPayloadBoundsElement<TPayloadType, T >& Elem = Elems[StartIndex + MaskIndex];
					if (PrePreFilterHelper(Elem.Payload, Visitor))
					{
						continue;
					}
					const FAABB3 InstanceBounds(Elem.Bounds.Min(), Elem.Bounds.Max());
					TSpatialVisitorData<TPayloadType> VisitData(Elem.Payload, true, InstanceBounds);
					if (Visitor.VisitOverlap(VisitData) == false)
					{
						return false;
					}
				}
			}
		}

		for (int32 SlowIter = SimdIters * 4; SlowIter < NumElems; ++SlowIter)
		{
			const TPayloadBoundsElement<TPayloadType, T >& Elem = Elems[SlowIter];
			if (PrePreFilterHelper(Elem.Payload, Visitor))
			{
				continue;
			}

			if (Elem.Bounds.Intersects(QueryBounds))
			{
				TSpatialVisitorData<TPayloadType> VisitData(Elem.Payload, true, FAABB3(Elem.Bounds.Min(), Elem.Bounds.Max()));
				if (Visitor.VisitOverlap(VisitData) == false)
				{
					return false;
				}
			}
		}

		return true;
	}

	template <bool bSweep, typename TQueryFastData, typename TSQVisitor>
	FORCEINLINE_DEBUGGABLE bool RaycastSweepImp(const TVec3<T>& Start, TQueryFastData& QueryFastData, const TVec3<T>& QueryHalfExtents, TSQVisitor& Visitor, const TVec3<T>& Dir, const TVec3<T> InvDir, const bool bParallel[3]) const
	{
		PHYSICS_CSV_CUSTOM_VERY_EXPENSIVE(PhysicsCounters, MaxLeafSize, Elems.Num(), ECsvCustomStatOp::Max);
		FReal TOI;
		for (const auto& Elem : Elems)
		{
			if (PrePreFilterHelper(Elem.Payload, Visitor))
			{
				continue;
			}

			const FAABB3 InstanceBounds(Elem.Bounds.Min(), Elem.Bounds.Max());
			if (TAABBTreeIntersectionHelper<TQueryFastData, bSweep ? EAABBQueryType::Sweep :
				EAABBQueryType::Raycast>::Intersects(Start, QueryFastData, TOI, InstanceBounds, FAABB3(), QueryHalfExtents, Dir, InvDir, bParallel))
			{
				TSpatialVisitorData<TPayloadType> VisitData(Elem.Payload, true, InstanceBounds);
				const bool bContinue = (bSweep && Visitor.VisitSweep(VisitData, QueryFastData)) || (!bSweep && Visitor.VisitRaycast(VisitData, QueryFastData));
				if (!bContinue)
				{
					return false;
				}
			}
		}
		return true;
	}

	template <typename TQueryFastData, typename TSQVisitor>
	FORCEINLINE_DEBUGGABLE bool RaycastImpSimd(const VectorRegister4Double& Start, TQueryFastData& QueryFastData, TSQVisitor& Visitor, const VectorRegister4Double& InvDir, const VectorRegister4Double& Parallel, const VectorRegister4Double& Length) const
	{
		PHYSICS_CSV_CUSTOM_VERY_EXPENSIVE(PhysicsCounters, MaxLeafSize, Elems.Num(), ECsvCustomStatOp::Max);
		VectorRegister4Double TOI;
		for (const auto& Elem : Elems)
		{
			const FAABBVectorizedDouble InstanceBounds(Elem.Bounds);
			if (InstanceBounds.RaycastFast(Start, InvDir, Parallel, Length, TOI))
			{
				if (PrePreFilterHelper(Elem.Payload, Visitor))
				{
					continue;
				}
				TSpatialVisitorData<TPayloadType> VisitData(Elem.Payload, true, FAABB3(Elem.Bounds.Min(), Elem.Bounds.Max()));
				const bool bContinue = Visitor.VisitRaycast(VisitData, QueryFastData);
				if (!bContinue)
				{
					return false;
				}
			}
		}
		return true;
	}


	void RemoveElement(TPayloadType Payload)
	{
		for (int32 Idx = 0; Idx < Elems.Num(); ++Idx)
		{
			if (Elems[Idx].Payload == Payload)
			{
				Elems.RemoveAtSwap(Idx);
				break;
			}
			if (UNLIKELY(!ensure(Idx != Elems.Num() - 1))) // Make sure the payload was actually in here
			{
				UE_LOGF(LogChaos, Warning, "AABBTree: Element not removed");
			}
		}
		bDirtyLeaf = true;
	}

	void UpdateElement(const TPayloadType& Payload, const TAABB<T, 3>& NewBounds, bool bHasBounds)
	{
		if (!bHasBounds)
			return;

		for (int32 Idx = 0; Idx < Elems.Num(); ++Idx)
		{
			if (Elems[Idx].Payload == Payload)
			{
				Elems[Idx].Bounds = NewBounds;
				UpdateElementHelper(Elems[Idx].Payload, Payload);
				break;
			}
		}
		bDirtyLeaf = true;
	}

	void UpdateElementWithoutDirty(const TPayloadType& Payload, const TAABB<T, 3>& NewBounds)
	{
		for (int32 Idx = 0; Idx < Elems.Num(); ++Idx)
		{
			if (Elems[Idx].Payload == Payload)
			{
				Elems[Idx].Bounds = NewBounds;
				UpdateElementHelper(Elems[Idx].Payload, Payload);
				break;
			}
		}
	}

	void AddElement(const TPayloadBoundsElement<TPayloadType, T>& Element)
	{
		Elems.Add(Element);
		bDirtyLeaf = true;
	}

	void Reset()
	{
		Elems.Reset();
		bDirtyLeaf = false;
	}

#if !UE_BUILD_SHIPPING
	void DebugDrawLeaf(ISpacialDebugDrawInterface<T>& InInterface, const FLinearColor& InLinearColor, float InThickness) const
	{
		const TAABB<T, 3> LeafBounds = TBoundsWrapperHelper<TPayloadType, T, bComputeBounds>::GetBounds();

		const float Alpha = (float)Elems.Num() / 10.f;
		const FLinearColor ColorByCount = FLinearColor::Green * (1.f - Alpha) + FLinearColor::Red * Alpha;
		const FVec3 ColorAsVec = { ColorByCount.R, ColorByCount.G, ColorByCount.B };
		
		InInterface.Box(LeafBounds, ColorAsVec, InThickness);
		for (const auto& Elem : Elems)
		{
			InInterface.Line(LeafBounds.Center(), Elem.Bounds.Center(), ColorAsVec, InThickness);
			InInterface.Box(Elem.Bounds, { (T)1.0, (T)0.2, (T)0.2 }, 1.0f);
		}
	}
#endif

	/** Print leaf information (bounds) for debugging purpose*/
	void PrintLeaf() const
	{
		int32 ElemIndex = 0;
		for (const auto& Elem : Elems)
		{
			UE_LOGF(LogChaos, Log, "Elem[%d] with bounds = %f %f %f | %f %f %f", ElemIndex, 
					Elem.Bounds.Min()[0], Elem.Bounds.Min()[1], Elem.Bounds.Min()[2], 
					Elem.Bounds.Max()[0], Elem.Bounds.Max()[1], Elem.Bounds.Max()[2]);
			++ElemIndex;
		}
	}

	void Serialize(FChaosArchive& Ar)
	{
		Ar << Elems;
	}

	TArray<TPayloadBoundsElement<TPayloadType, T>> Elems;

	/** Flag on the leaf to know if it has been updated */
	bool bDirtyLeaf = false;

	friend ::FChaosVDDataWrapperUtils;
};

template <typename TPayloadType, bool bComputeBounds, typename T>
FChaosArchive& operator<<(FChaosArchive& Ar, TAABBTreeLeafArray<TPayloadType, bComputeBounds, T>& LeafArray)
{
	LeafArray.Serialize(Ar);
	return Ar;
}


// Default container behaviour is a that of a TArray
template<typename LeafType>
class TLeafContainer  : public TArray<LeafType>
{
public:
	void Serialize(FChaosArchive& Ar)
	{
		Ar << *static_cast<TArray<LeafType>*>(this);
	}
};


// Here we are specializing the behaviour of our leaf container to minimize memory allocations/deallocations when the container is reset. 
// This is accomplished by only resetting the leafArrays and not the whole container
// This is only specialized for one specific type, but can expanded to more types if necessary
template<>
class TLeafContainer<TAABBTreeLeafArray<FAccelerationStructureHandle, true, FReal >> : private TArray<TAABBTreeLeafArray<FAccelerationStructureHandle, true, FReal>>
{
	
private:
	typedef TAABBTreeLeafArray<FAccelerationStructureHandle, true, FReal> FLeafType;
	using FParent = TArray<FLeafType>;
public:

	using ElementType = FParent::ElementType;
	

	FLeafType& operator[](SizeType Index)
	{
		return FParent::operator[](Index);
	}
	const FLeafType& operator[](SizeType Index) const
	{
		return FParent::operator[](Index);
	}
	SizeType Num() const
	{
		return NumOfValidElements;
	}
	void Reserve(SizeType Number)
	{
		FParent::Reserve(Number);
	}
	void Reset()
	{
		for (int32 ElementIndex = 0; ElementIndex < NumOfValidElements; ElementIndex++)
		{
			(*this)[ElementIndex].Reset();
		}
		NumOfValidElements = 0;		
	}
	SizeType Add(const FLeafType& Item)
	{
		NumOfValidElements++;
		if (NumOfValidElements > FParent::Num())
		{
			FParent::Add(Item);
		}
		else
		{
			(*this)[NumOfValidElements - 1] = Item;
		}		
		return NumOfValidElements - 1;
	}

	void Serialize(FChaosArchive& Ar)
	{
		ensure(false); // This type does not get serialized
	}
private:
	int32 NumOfValidElements = 0;

	friend ::FChaosVDDataWrapperUtils;
};

template <typename LeafType>
FChaosArchive& operator<<(FChaosArchive& Ar, TLeafContainer<LeafType>& LeafArray)
{
	LeafArray.Serialize(Ar);

	return Ar;
}

template <typename T>
struct TAABBTreeNode
{
	TAABBTreeNode()
		: ChildrenBounds{TAABB<T, 3>() , TAABB<T, 3>()}
		, ChildrenNodes{INDEX_NONE, INDEX_NONE}
		, ParentNode(INDEX_NONE)
		, bLeaf(false)
		, bDirtyNode(false)
	{}

	TAABB<T, 3> ChildrenBounds[2];
	int32 ChildrenNodes[2];
	int32 ParentNode;
	bool bLeaf : 1;
	bool bDirtyNode : 1;

#if !UE_BUILD_SHIPPING
	void DebugDraw(ISpacialDebugDrawInterface<T>& InInterface, const TArray<TAABBTreeNode<T>>& Nodes, const FVec3& InLinearColor, float InThickness) const
	{
		constexpr float ColorRatio = 0.75f;
		constexpr float LineThicknessRatio = 0.75f;
		if (!bLeaf)
		{
			FLinearColor ChildColor = FLinearColor::MakeRandomColor();
			for (int ChildIndex = 0; ChildIndex < 2; ++ChildIndex)
			{
				int32 NodeIndex = ChildrenNodes[ChildIndex];
				if (NodeIndex > 0 && NodeIndex < Nodes.Num())
				{
					Nodes[NodeIndex].DebugDraw(InInterface, Nodes, { ChildColor.R, ChildColor.G, ChildColor.B }, InThickness * LineThicknessRatio);
				}
			}
			for (int ChildIndex = 0; ChildIndex < 2; ++ChildIndex)
			{
				InInterface.Box(ChildrenBounds[ChildIndex], InLinearColor, InThickness);
			}
		}
	}
#endif

	void Serialize(FChaosArchive& Ar)
	{
		for (auto& Bounds : ChildrenBounds)
		{
			TBox<T, 3>::SerializeAsAABB(Ar, Bounds);
		}

		for (auto& Node : ChildrenNodes)
		{
			Ar << Node;
		}

		// Making a copy here because bLeaf is a bitfield and the archive can't handle it
		bool bLeafCopy = bLeaf;
		Ar << bLeafCopy;

		// Dynamic trees are not serialized
		if (Ar.IsLoading())
		{
			ParentNode = INDEX_NONE;
			bLeaf = bLeafCopy;
		}
	}
};

template <typename T>
FORCEINLINE FChaosArchive& operator<<(FChaosArchive& Ar, TAABBTreeNode<T>& Node)
{
	Node.Serialize(Ar);
	return Ar;
}

struct FAABBTreePayloadInfo
{
	int32 GlobalPayloadIdx;
	int32 DirtyPayloadIdx;
	int32 LeafIdx;
	int32 DirtyGridOverflowIdx;
	int32 NodeIdx;

	FAABBTreePayloadInfo(int32 InGlobalPayloadIdx = INDEX_NONE, int32 InDirtyIdx = INDEX_NONE, int32 InLeafIdx = INDEX_NONE, int32 InDirtyGridOverflowIdx = INDEX_NONE, int32 InNodeIdx = INDEX_NONE)
		: GlobalPayloadIdx(InGlobalPayloadIdx)
		, DirtyPayloadIdx(InDirtyIdx)
		, LeafIdx(InLeafIdx)
		, DirtyGridOverflowIdx(InDirtyGridOverflowIdx)
		, NodeIdx(InNodeIdx)
	{}

	void Serialize(FArchive& Ar)
	{
		Ar << GlobalPayloadIdx;
		Ar << DirtyPayloadIdx;
		Ar << LeafIdx;
		Ar << DirtyGridOverflowIdx;
		// Dynamic trees are not serialized
		if (Ar.IsLoading())
		{
			NodeIdx = INDEX_NONE;
		}
	}
};

FORCEINLINE FArchive& operator<<(FArchive& Ar, FAABBTreePayloadInfo& PayloadInfo)
{
	PayloadInfo.Serialize(Ar);
	return Ar;
}

extern CHAOS_API int32 MaxDirtyElements;

struct DirtyGridHashEntry
{
	DirtyGridHashEntry()
	{
		Index = 0;
		Count = 0;
	}

	DirtyGridHashEntry(const DirtyGridHashEntry& Other)
	{
		Index = Other.Index;
		Count = Other.Count;
	}

	int32 Index;  // Index into FlattenedCellArrayOfDirtyIndices
	int32 Count;  // Number of valid entries from Index in FlattenedCellArrayOfDirtyIndices
};

template<typename PayloadType>
struct TDefaultAABBTreeStorageTraits
{
	using PayloadToInfoType = TArrayAsMap<PayloadType, FAABBTreePayloadInfo>;

	static void InitPayloadToInfo(PayloadToInfoType& PayloadToInfo)
	{}
};

template <typename TPayloadType, typename TLeafType, bool bMutable = true, typename T = FReal, typename StorageTraits = TDefaultAABBTreeStorageTraits<TPayloadType>>
class TAABBTree final : public ISpatialAcceleration<TPayloadType, T, 3> 
{
private:
	using FElement = TPayloadBoundsElement<TPayloadType, T>;
	using FNode = TAABBTreeNode<T>;
public:
	using PayloadType = TPayloadType;
	static constexpr int D = 3;
	using TType = T;
	static constexpr T DefaultMaxPayloadBounds = 100000;
	static constexpr int32 DefaultMaxChildrenInLeaf = 12;
	static constexpr int32 DefaultMaxTreeDepth = 16;
	static constexpr int32 DefaultMaxNumToProcess = 0; // 0 special value for processing all without timeslicing
	static constexpr ESpatialAcceleration StaticType = std::is_same_v<TAABBTreeLeafArray<TPayloadType>, TLeafType> ? ESpatialAcceleration::AABBTree :
		(std::is_same_v<TBoundingVolume<TPayloadType>, TLeafType> ? ESpatialAcceleration::AABBTreeBV : ESpatialAcceleration::Unknown);
	TAABBTree()
		: ISpatialAcceleration<TPayloadType, T, 3>(StaticType)
		, bDynamicTree(false)
		, RootNode(INDEX_NONE)
		, FirstFreeInternalNode(INDEX_NONE)
		, FirstFreeLeafNode(INDEX_NONE)
		, MaxChildrenInLeaf(DefaultMaxChildrenInLeaf)
		, MaxTreeDepth(DefaultMaxTreeDepth)
		, MaxPayloadBounds(DefaultMaxPayloadBounds)
		, MaxNumToProcess(DefaultMaxNumToProcess)
		, bModifyingTreeMultiThreadingFastCheck(false)
		, bShouldRebuild(true)
		, bBuildOverlapCache(true)		
	{
		GetCVars();

		StorageTraits::InitPayloadToInfo(PayloadToInfo);
	}

	virtual void Reset() override
	{
		Nodes.Reset();
		Leaves.Reset();
		DirtyElements.Reset();
		CellHashToFlatArray.Reset();
		FlattenedCellArrayOfDirtyIndices.Reset();
		DirtyElementsGridOverflow.Reset();
		TreeStats.Reset();
		TreeExpensiveStats.Reset();
		GlobalPayloads.Reset();
		PayloadToInfo.Reset();
		
		OverlappingLeaves.Reset();
		OverlappingOffsets.Reset();
		OverlappingPairs.Reset();
		OverlappingCounts.Reset();
		
		NumProcessedThisSlice = 0;

		StartSliceTimeStamp = 0.0;
		CurrentDataElementsCopiedSinceLastCheck = 0;
		CurrentProcessedNodesSinceChecked = 0;
		
		WorkStack.Reset();
		WorkPoolFreeList.Reset();
		WorkPool.Reset();

		bShouldRebuild = true;

		RootNode = INDEX_NONE;
		FirstFreeInternalNode = INDEX_NONE;
		FirstFreeLeafNode = INDEX_NONE;

		this->SetAsyncTimeSlicingComplete(true);

		if (DirtyElementTree != nullptr)
		{
			DirtyElementTree->Reset();
		}
	}

	virtual void ProgressAsyncTimeSlicing(bool ForceBuildCompletion) override
	{
		SCOPE_CYCLE_COUNTER(STAT_AABBTreeProgressTimeSlice);

		if (bDynamicTree)
		{
			// Nothing to do
			this->SetAsyncTimeSlicingComplete(true);
			return;
		}

		// force is to stop time slicing and complete the rest of the build now
		if (ForceBuildCompletion)
		{
			MaxNumToProcess = 0;
		}

		// still has work to complete
		if (WorkStack.Num())
		{
			NumProcessedThisSlice = 0;
			StartSliceTimeStamp = FPlatformTime::Seconds();
			SplitNode();
		}
	}

	template <typename TParticles>
	TAABBTree(const TParticles& Particles, int32 InMaxChildrenInLeaf = DefaultMaxChildrenInLeaf, int32 InMaxTreeDepth = DefaultMaxTreeDepth, T InMaxPayloadBounds = DefaultMaxPayloadBounds, int32 InMaxNumToProcess = DefaultMaxNumToProcess, bool bInDynamicTree = false, bool bInUseDirtyTree = false, bool bInBuildOverlapCache = true)
		: ISpatialAcceleration<TPayloadType, T, 3>(StaticType)
		, bDynamicTree(bInDynamicTree)
		, MaxChildrenInLeaf(InMaxChildrenInLeaf)
		, MaxTreeDepth(InMaxTreeDepth)
		, MaxPayloadBounds(InMaxPayloadBounds)
		, MaxNumToProcess(InMaxNumToProcess)
		, bModifyingTreeMultiThreadingFastCheck(false)
		, bShouldRebuild(true)
		, bBuildOverlapCache(bInBuildOverlapCache)		
	{
		if (bInUseDirtyTree)
		{
			DirtyElementTree = TUniquePtr<TAABBTree>(new TAABBTree());
			DirtyElementTree->SetTreeToDynamic();
		}

		StorageTraits::InitPayloadToInfo(PayloadToInfo);

		GenerateTree(Particles);
	}

	// Tag dispatch enable for the below constructor to allow setting up the defaults without an initial set of particles
	struct EmptyInit {};

	TAABBTree(EmptyInit, int32 InMaxChildrenInLeaf = DefaultMaxChildrenInLeaf, int32 InMaxTreeDepth = DefaultMaxTreeDepth, T InMaxPayloadBounds = DefaultMaxPayloadBounds, int32 InMaxNumToProcess = DefaultMaxNumToProcess, bool bInDynamicTree = false, bool bInUseDirtyTree = false, bool bInBuildOverlapCache = true)
		: ISpatialAcceleration<TPayloadType, T, 3>(StaticType)
		, bDynamicTree(bInDynamicTree)
		, MaxChildrenInLeaf(InMaxChildrenInLeaf)
		, MaxTreeDepth(InMaxTreeDepth)
		, MaxPayloadBounds(InMaxPayloadBounds)
		, MaxNumToProcess(InMaxNumToProcess)
		, bModifyingTreeMultiThreadingFastCheck(false)
		, bShouldRebuild(true)
		, bBuildOverlapCache(bInBuildOverlapCache)
	{
		if(bInUseDirtyTree)
		{
			DirtyElementTree = TUniquePtr<TAABBTree>(new TAABBTree());
			DirtyElementTree->SetTreeToDynamic();
		}

		StorageTraits::InitPayloadToInfo(PayloadToInfo);
	}

	template <typename ParticleView>
	void Reinitialize(const ParticleView& Particles, int32 InMaxChildrenInLeaf = DefaultMaxChildrenInLeaf, int32 InMaxTreeDepth = DefaultMaxTreeDepth, T InMaxPayloadBounds = DefaultMaxPayloadBounds, int32 InMaxNumToProcess = DefaultMaxNumToProcess, bool bInDynamicTree = false, bool bInbBuildOverlapCache = true)
	{
		bDynamicTree = bInDynamicTree;
		MaxChildrenInLeaf = InMaxChildrenInLeaf;
		MaxTreeDepth = InMaxTreeDepth;
		MaxPayloadBounds = InMaxPayloadBounds;
		MaxNumToProcess = InMaxNumToProcess;
		bModifyingTreeMultiThreadingFastCheck = false;
		bShouldRebuild = true;
		bBuildOverlapCache = bInbBuildOverlapCache;		
		GenerateTree(Particles);
	}

	virtual TArray<TPayloadType> FindAllIntersections(const FAABB3& Box) const override { return FindAllIntersectionsImp(Box); }

	bool GetAsBoundsArray(TArray<TAABB<T, 3>>& AllBounds, int32 NodeIdx, int32 ParentNode, TAABB<T, 3>& Bounds)
	{
		if (Nodes[NodeIdx].bLeaf)
		{
			AllBounds.Add(Bounds);
			return false;
		}
		else
		{
			GetAsBoundsArray(AllBounds, Nodes[NodeIdx].ChildrenNodes[0], NodeIdx, Nodes[NodeIdx].ChildrenBounds[0]);
			GetAsBoundsArray(AllBounds, Nodes[NodeIdx].ChildrenNodes[1], NodeIdx, Nodes[NodeIdx].ChildrenBounds[0]);
		}
		return true;
	}

	virtual ~TAABBTree() = default;

	void CopyFrom(const TAABBTree& Other)
	{
		(*this) = Other;
	}

	virtual TUniquePtr<ISpatialAcceleration<TPayloadType, T, 3>> Copy() const override
	{
		return TUniquePtr<ISpatialAcceleration<TPayloadType, T, 3>>(new TAABBTree(*this));
	}

	virtual void Raycast(const FVec3& Start, const FVec3& Dir, const FReal Length, ISpatialVisitor<TPayloadType, FReal>& Visitor) const override
	{
		TSpatialVisitor<TPayloadType, FReal> ProxyVisitor(Visitor);
		Raycast(Start, Dir, Length, ProxyVisitor);
	}

	template <typename SQVisitor>
	void Raycast(const FVec3& Start, const FVec3& Dir, const FReal Length, SQVisitor& Visitor) const
	{
		FQueryFastData QueryFastData(Dir, Length);
		QueryImp<EAABBQueryType::Raycast>(Start, QueryFastData, FVec3(), FAABB3(), Visitor, QueryFastData.Dir, QueryFastData.InvDir, QueryFastData.bParallel);
	}

	template <typename SQVisitor>
	bool RaycastFast(const FVec3& Start, FQueryFastData& CurData, SQVisitor& Visitor, const FVec3& Dir, const FVec3 InvDir, const bool bParallel[3]) const
	{
		return QueryImp<EAABBQueryType::Raycast>(Start, CurData, TVec3<T>(), TAABB<T, 3>(), Visitor, Dir, InvDir, bParallel);
	}

	void Sweep(const FVec3& Start, const FVec3& Dir, const FReal Length, const FVec3 QueryHalfExtents, ISpatialVisitor<TPayloadType, FReal>& Visitor) const override
	{
		TSpatialVisitor<TPayloadType, FReal> ProxyVisitor(Visitor);
		Sweep(Start, Dir, Length, QueryHalfExtents, ProxyVisitor);
	}

	template <typename SQVisitor>
	void Sweep(const FVec3& Start, const FVec3& Dir, const FReal Length, const FVec3 QueryHalfExtents, SQVisitor& Visitor) const
	{
		FQueryFastData QueryFastData(Dir, Length);
		QueryImp<EAABBQueryType::Sweep>(Start, QueryFastData, QueryHalfExtents, FAABB3(), Visitor, QueryFastData.Dir, QueryFastData.InvDir, QueryFastData.bParallel);
	}

	template <typename SQVisitor>
	bool SweepFast(const FVec3& Start, FQueryFastData& CurData, const FVec3 QueryHalfExtents, SQVisitor& Visitor, const FVec3& Dir, const FVec3 InvDir, const bool bParallel[3]) const
	{
		return QueryImp<EAABBQueryType::Sweep>(Start,CurData, QueryHalfExtents, FAABB3(), Visitor, Dir, InvDir, bParallel);
	}

	void Overlap(const FAABB3& QueryBounds, ISpatialVisitor<TPayloadType, FReal>& Visitor) const override
	{
		TSpatialVisitor<TPayloadType, FReal> ProxyVisitor(Visitor);
		Overlap(QueryBounds, ProxyVisitor);
	}

	template <typename SQVisitor>
	void Overlap(const FAABB3& QueryBounds, SQVisitor& Visitor) const
	{
		OverlapFast(QueryBounds, Visitor);
	}

	template <typename SQVisitor>
	bool OverlapFast(const FAABB3& QueryBounds, SQVisitor& Visitor) const
	{
		//dummy variables to reuse templated path
		FQueryFastDataVoid VoidData;
		return QueryImp<EAABBQueryType::Overlap>(FVec3(), VoidData, FVec3(), QueryBounds, Visitor, VoidData.Dir, VoidData.InvDir, VoidData.bParallel);
	}

	// This is to make sure important parameters are not changed during inopportune times
	void GetCVars()
	{
		DirtyElementGridCellSize = (T) FAABBTreeDirtyGridCVars::DirtyElementGridCellSize;
		if (DirtyElementGridCellSize > UE_SMALL_NUMBER)
		{
			DirtyElementGridCellSizeInv = 1.0f / DirtyElementGridCellSize;
		}
		else
		{
			DirtyElementGridCellSizeInv = 1.0f;
		}

		DirtyElementMaxGridCellQueryCount = FAABBTreeDirtyGridCVars::DirtyElementMaxGridCellQueryCount;
		DirtyElementMaxPhysicalSizeInCells = FAABBTreeDirtyGridCVars::DirtyElementMaxPhysicalSizeInCells;
		DirtyElementMaxCellCapacity = FAABBTreeDirtyGridCVars::DirtyElementMaxCellCapacity;
	}

	FORCEINLINE_DEBUGGABLE bool DirtyElementGridEnabled() const
	{
		return DirtyElementTree == nullptr && DirtyElementGridCellSize > 0.0f &&
			DirtyElementMaxGridCellQueryCount > 0 &&
			DirtyElementMaxPhysicalSizeInCells > 0 &&
			DirtyElementMaxCellCapacity > 0;
	}

	FORCEINLINE_DEBUGGABLE bool EnoughSpaceInGridCell(int32 Hash)
	{
		DirtyGridHashEntry* HashEntry = CellHashToFlatArray.Find(Hash);
		if (HashEntry)
		{
			if (HashEntry->Count >= DirtyElementMaxCellCapacity) // Checking if we are at capacity
			{
				return false;
			}
		}

		return true;
	}

	// Returns true if there was enough space in the cell to add the new dirty element index or if the element was already added (This second condition should not happen)
	//(The second condition should never be true for the current implementation)
	FORCEINLINE_DEBUGGABLE bool AddNewDirtyParticleIndexToGridCell(int32 Hash, int32 NewDirtyIndex)
	{
		DirtyGridHashEntry* HashEntry = CellHashToFlatArray.Find(Hash);
		if (HashEntry)
		{
			if (HashEntry->Count < DirtyElementMaxCellCapacity)
			{
				if (ensure(InsertValueIntoSortedSubArray(FlattenedCellArrayOfDirtyIndices, NewDirtyIndex, HashEntry->Index, HashEntry->Count)))
				{
					++(HashEntry->Count);
				}
				return true;
			}
		}
		else
		{
			DirtyGridHashEntry& NewHashEntry = CellHashToFlatArray.Add(Hash);
			NewHashEntry.Index = FlattenedCellArrayOfDirtyIndices.Num(); // End of flat array
			NewHashEntry.Count = 1;
			FlattenedCellArrayOfDirtyIndices.AddUninitialized(DirtyElementMaxCellCapacity);
			FlattenedCellArrayOfDirtyIndices[NewHashEntry.Index] = NewDirtyIndex;
			TreeStats.StatNumNonEmptyCellsInGrid++;
			return true;
		}
		return false;
	}

	// Returns true if the dirty particle was in the grid and successfully deleted
	FORCEINLINE_DEBUGGABLE bool DeleteDirtyParticleIndexFromGridCell(int32 Hash, int32 DirtyIndex)
	{
		DirtyGridHashEntry* HashEntry = CellHashToFlatArray.Find(Hash);
		if (HashEntry && HashEntry->Count >= 1)
		{
			if (DeleteValueFromSortedSubArray(FlattenedCellArrayOfDirtyIndices, DirtyIndex, HashEntry->Index, HashEntry->Count))
			{
				--(HashEntry->Count);
				// Not deleting cell when it gets empty, it may get reused or will be deleted when the AABBTree is rebuilt
				return true;
			}
		}
		return false;
	}

	FORCEINLINE_DEBUGGABLE void DeleteDirtyParticleEverywhere(int32 DeleteDirtyParticleIdx, int32 DeleteDirtyGridOverflowIdx)
	{
		if (DeleteDirtyGridOverflowIdx == INDEX_NONE)
		{
			// Remove this element from the Grid
			DoForOverlappedCells(DirtyElements[DeleteDirtyParticleIdx].Bounds, DirtyElementGridCellSize, DirtyElementGridCellSizeInv, [&](int32 Hash) {
				ensure(DeleteDirtyParticleIndexFromGridCell(Hash, DeleteDirtyParticleIdx));
				return true;
				});
		}
		else
		{
			// remove element from the grid overflow
			ensure(DirtyElementsGridOverflow[DeleteDirtyGridOverflowIdx] == DeleteDirtyParticleIdx);

			if (DeleteDirtyGridOverflowIdx + 1 < DirtyElementsGridOverflow.Num())
			{
				auto LastOverflowPayload = DirtyElements[DirtyElementsGridOverflow.Last()].Payload;
				PayloadToInfo.FindChecked(LastOverflowPayload).DirtyGridOverflowIdx = DeleteDirtyGridOverflowIdx;
			}
			DirtyElementsGridOverflow.RemoveAtSwap(DeleteDirtyGridOverflowIdx);
		}

		if (DeleteDirtyParticleIdx + 1 < DirtyElements.Num())
		{
			// Now rename the last element in DirtyElements in both the grid and the overflow
			// So that it is correct after swapping Dirty elements in next step
			int32 LastDirtyElementIndex = DirtyElements.Num() - 1;
			auto LastDirtyPayload = DirtyElements[LastDirtyElementIndex].Payload;
			int32 LastDirtyGridOverflowIdx = PayloadToInfo.FindChecked(LastDirtyPayload).DirtyGridOverflowIdx;
			if (LastDirtyGridOverflowIdx == INDEX_NONE)
			{
				// Rename this element in the Grid
				DoForOverlappedCells(DirtyElements.Last().Bounds, DirtyElementGridCellSize, DirtyElementGridCellSizeInv, [&](int32 Hash) {
					ensure(DeleteDirtyParticleIndexFromGridCell(Hash, LastDirtyElementIndex));
					ensure(AddNewDirtyParticleIndexToGridCell(Hash, DeleteDirtyParticleIdx));
					return true;
					});
			}
			else
			{
				// Rename element in overflow instead
				DirtyElementsGridOverflow[LastDirtyGridOverflowIdx] = DeleteDirtyParticleIdx;
			}

			// Copy the Payload to the new index
			
			PayloadToInfo.FindChecked(LastDirtyPayload).DirtyPayloadIdx = DeleteDirtyParticleIdx;
		}
		DirtyElements.RemoveAtSwap(DeleteDirtyParticleIdx);
	}

	FORCEINLINE_DEBUGGABLE int32 AddDirtyElementToGrid(const TAABB<T, 3>& NewBounds, int32 NewDirtyElement)
	{
		bool bAddToGrid = !TooManyOverlapQueryCells(NewBounds, DirtyElementGridCellSizeInv, DirtyElementMaxPhysicalSizeInCells);
		if (!bAddToGrid)
		{
			TreeStats.StatNumElementsTooLargeForGrid++;
		}

		if (bAddToGrid)
		{
			DoForOverlappedCells(NewBounds, DirtyElementGridCellSize, DirtyElementGridCellSizeInv, [&](int32 Hash) {
				if (!EnoughSpaceInGridCell(Hash))
				{
					bAddToGrid = false;
					return false; // early exit to avoid going through all the cells
				}
				return true;
				});
		}

		if (bAddToGrid)
		{
			DoForOverlappedCells(NewBounds, DirtyElementGridCellSize, DirtyElementGridCellSizeInv, [&](int32 Hash) {
				ensure(AddNewDirtyParticleIndexToGridCell(Hash, NewDirtyElement));
				return true;
				});
		}
		else
		{
			int32 NewOverflowIndex = DirtyElementsGridOverflow.Add(NewDirtyElement);
			return NewOverflowIndex;
		}

		return INDEX_NONE;
	}

	FORCEINLINE_DEBUGGABLE int32 UpdateDirtyElementInGrid(const TAABB<T, 3>& NewBounds, int32 DirtyElementIndex, int32 DirtyGridOverflowIdx)
	{
		if (DirtyGridOverflowIdx == INDEX_NONE)
		{
			const TAABB<T, 3>& OldBounds = DirtyElements[DirtyElementIndex].Bounds;

			// Delete element in cells that are no longer overlapping
			DoForOverlappedCellsExclude(OldBounds, NewBounds, DirtyElementGridCellSize, DirtyElementGridCellSizeInv, [&](int32 Hash) -> bool {
				ensure(DeleteDirtyParticleIndexFromGridCell(Hash, DirtyElementIndex));
				return true;
				});

			// Add to new overlapped cells
			if (!DoForOverlappedCellsExclude(NewBounds, OldBounds, DirtyElementGridCellSize, DirtyElementGridCellSizeInv, [&](int32 Hash) -> bool {
					return AddNewDirtyParticleIndexToGridCell(Hash, DirtyElementIndex);
				}))
			{
				// Was not able to add it to the grid , so delete element from grid
				DoForOverlappedCells(NewBounds, DirtyElementGridCellSize, DirtyElementGridCellSizeInv, [&](int32 Hash) {
						DeleteDirtyParticleIndexFromGridCell(Hash, DirtyElementIndex);
						return true;
					});
				// Add to overflow
				int32 NewOverflowIndex = DirtyElementsGridOverflow.Add(DirtyElementIndex);
				return NewOverflowIndex;
			}
		}
		return DirtyGridOverflowIdx;
	}

	struct FElementsCollection
	{
		TArray<FElement> AllElements;
		int32 DepthTotal;
		int32 MaxDepth;
		int32 DirtyElementCount;
	};

	FElementsCollection DebugGetElementsCollection() const 
	{
		TArray<FElement> AllElements;
		for(int LeafIndex = 0; LeafIndex < Leaves.Num(); LeafIndex++)
		{
			const TLeafType& Leaf = Leaves[LeafIndex];
			Leaf.GatherElements(AllElements);
		}

		int32 MaxDepth = 0;
		int32 DepthTotal = 0;
		for(const FElement& Element : AllElements)
		{
			const FAABBTreePayloadInfo* PayloadInfo = PayloadToInfo.Find(Element.Payload);

			if(!PayloadInfo)
			{
				continue;
			}

			int32 Depth = 0;
			int32 Node = PayloadInfo->NodeIdx;

			while(Node != INDEX_NONE)
			{
				Node = Nodes[Node].ParentNode;
				if(Node != INDEX_NONE)
				{
					Depth++;
				}
			}
			if(Depth > MaxDepth)
			{
				MaxDepth = Depth;
			}
			DepthTotal += Depth;
		}

		return { AllElements, DepthTotal, MaxDepth, DirtyElements.Num() };
	}

	// Expensive function: Don't call unless debugging
	void DynamicTreeDebugStats() const
	{
		const FElementsCollection ElemData = DebugGetElementsCollection();

#if !WITH_EDITOR
		CSV_CUSTOM_STAT(ChaosPhysicsTimers, MaximumTreeDepth, ElemData.MaxDepth, ECsvCustomStatOp::Max);
		CSV_CUSTOM_STAT(ChaosPhysicsTimers, AvgTreeDepth, ElemData.DepthTotal / ElemData.AllElements.Num(), ECsvCustomStatOp::Max);
		CSV_CUSTOM_STAT(ChaosPhysicsTimers, Dirty, ElemData.DirtyElementCount, ECsvCustomStatOp::Max);
#endif

	}

#if !UE_BUILD_SHIPPING
	void DumpStats() const override
	{
		if(GLog)
		{
			DumpStatsTo(*GLog);
		}
	}

	void DumpStatsTo(FOutputDevice& Ar) const override
	{
		const FElementsCollection ElemData = DebugGetElementsCollection();

		const int32 ElementsNum = ElemData.AllElements.Num();
		const int32 PayloadMapNum = PayloadToInfo.Num();
		const int32 PayloadMapCapacity = PayloadToInfo.Capacity();
		const uint64 PayloadAllocsize = (uint64)PayloadToInfo.GetAllocatedSize();
		const float AvgDepth = ElementsNum > 0 ? (float)ElemData.DepthTotal / (float)ElementsNum : 0.0f;

		Ar.Logf(ELogVerbosity::Log, TEXT("\t\tContains %d elements"), ElementsNum);
		Ar.Logf(ELogVerbosity::Log, TEXT("\t\tMax depth is %d"), ElemData.MaxDepth);
		Ar.Logf(ELogVerbosity::Log, TEXT("\t\tAvg depth is %.3f"), AvgDepth);
		Ar.Logf(ELogVerbosity::Log, TEXT("\t\tDirty element count is %d"), ElemData.DirtyElementCount);
		Ar.Logf(ELogVerbosity::Log, TEXT("\t\tPayload container size is %d elements"), PayloadMapNum);
		Ar.Logf(ELogVerbosity::Log, TEXT("\t\tPayload container capacity is %d elements"), PayloadMapCapacity);
		Ar.Logf(ELogVerbosity::Log, TEXT("\t\tAllocated size of payload container is %u bytes (%u per tree element)"), PayloadAllocsize, ElementsNum > 0 ? PayloadAllocsize / (uint32)ElementsNum : 0);
		
		if(DirtyElementTree)
		{
			Ar.Logf(ELogVerbosity::Log, TEXT(""));
			Ar.Logf(ELogVerbosity::Log, TEXT("\t\tDirty Tree:"));
			DirtyElementTree->DumpStatsTo(Ar);
		}
	}
#endif

	int32 AllocateInternalNode()
	{
		int32 AllocatedNodeIdx = FirstFreeInternalNode;
		if (FirstFreeInternalNode != INDEX_NONE)
		{
			// Unlink from free list
			FirstFreeInternalNode = Nodes[FirstFreeInternalNode].ChildrenNodes[1];
		}
		else
		{
			// create the actual node space
			AllocatedNodeIdx = Nodes.AddUninitialized(1);;
			Nodes[AllocatedNodeIdx].bLeaf = false;
		}
		if (UNLIKELY(!ensure(Nodes[AllocatedNodeIdx].bLeaf == false)))
		{
			UE_LOGF(LogChaos, Warning, "AABBTree: Allocated internal node is a leaf");
		}
		return AllocatedNodeIdx;
	}


	struct NodeAndLeafIndices
	{
		int32 NodeIdx;
		int32 LeafIdx;
	};

	NodeAndLeafIndices AllocateLeafNodeAndLeaf(const TPayloadType& Payload, const TAABB<T, 3>& NewBounds)
	{
		int32 AllocatedNodeIdx = FirstFreeLeafNode;
		int32 LeafIndex;
		if (FirstFreeLeafNode != INDEX_NONE)
		{
			FirstFreeLeafNode = Nodes[FirstFreeLeafNode].ChildrenNodes[1];
			LeafIndex = Nodes[AllocatedNodeIdx].ChildrenNodes[0]; // This is already set when it was allocated for the first time
			FElement NewElement{ Payload, NewBounds };
			Leaves[LeafIndex].AddElement(NewElement);
			Leaves[LeafIndex].RecomputeBounds(bDynamicTree);
		}
		else
		{
			LeafIndex = Leaves.Num();

			// create the actual node space
			AllocatedNodeIdx = Nodes.AddUninitialized(1);
			Nodes[AllocatedNodeIdx].ChildrenNodes[0] = LeafIndex;
			Nodes[AllocatedNodeIdx].bLeaf = true;

			FElement NewElement{ Payload, NewBounds };
			TArray<FElement> SingleElementArray;
			SingleElementArray.Add(NewElement);
			Leaves.Add(TLeafType{ SingleElementArray }); // Extra copy
		}

		// Expand the leaf node bounding box to reduce the number of updates
		TAABB<T, 3> ExpandedBounds = NewBounds;
		ExpandedBounds.Thicken(FAABBTreeCVars::DynamicTreeBoundingBoxPadding);
		Nodes[AllocatedNodeIdx].ChildrenBounds[0] = ExpandedBounds;

		Nodes[AllocatedNodeIdx].ParentNode = INDEX_NONE;
		if (UNLIKELY(!ensure(Nodes[AllocatedNodeIdx].bLeaf == true)))
		{
			UE_LOGF(LogChaos, Warning, "AABBTree: Allocated leaf node is not a leaf");
		}
		
		return NodeAndLeafIndices{ AllocatedNodeIdx , LeafIndex };
	}

	void DeAllocateInternalNode(int32 NodeIdx)
	{
		Nodes[NodeIdx].ChildrenNodes[1] = FirstFreeInternalNode;
		FirstFreeInternalNode = NodeIdx;
		if (UNLIKELY(!ensure(Nodes[NodeIdx].bLeaf == false)))
		{
			UE_LOGF(LogChaos, Warning, "AABBTree: Deallocated Internal node is a leaf");
		}
	}

	void  DeAllocateLeafNode(int32 NodeIdx)
	{
		
		Leaves[Nodes[NodeIdx].ChildrenNodes[0]].Reset();

		Nodes[NodeIdx].ChildrenNodes[1] = FirstFreeLeafNode;
		FirstFreeLeafNode = NodeIdx;
		if (UNLIKELY(!ensure(Nodes[NodeIdx].bLeaf == true)))
		{
			UE_LOGF(LogChaos, Warning, "AABBTree: Deallocated Leaf node is not a leaf");
		}
	}

	// Is the input node Child 0 or Child 1?
	int32 WhichChildAmI(int32 NodeIdx)
	{
		check(NodeIdx != INDEX_NONE);
		int32 ParentIdx = Nodes[NodeIdx].ParentNode;
		check(ParentIdx != INDEX_NONE);
		if (Nodes[ParentIdx].ChildrenNodes[0] == NodeIdx)
		{
			return  0;
		}
		else
		{
			if (UNLIKELY(!ensure(Nodes[ParentIdx].ChildrenNodes[1] == NodeIdx)))
			{
				UE_LOGF(LogChaos, Warning, "AABBTree: Child node not found");
			}
			return 1;
		}
	}

	// Is the input node Child 0 or Child 1?
	int32 GetSiblingIndex(int32 NodeIdx)
	{
		return(WhichChildAmI(NodeIdx) ^ 1);
	}

public:
	int32 FindBestSibling(const TAABB<T, 3>& InNewBounds, bool& bOutAddToLeaf)
	{
		bOutAddToLeaf = false;
		
		TAABB<T, 3> NewBounds = InNewBounds;
		NewBounds.Thicken(FAABBTreeCVars::DynamicTreeBoundingBoxPadding);
		const FReal NewBoundsArea = NewBounds.GetArea();

		//Priority Q of indices to explore
		PriorityQ.Reset();

		int32 QIndex = 0;
		
		// Initializing
		FNode& RNode = Nodes[RootNode];
		TAABB<T, 3> WorkingAABB{ NewBounds };
		WorkingAABB.GrowToInclude(RNode.ChildrenBounds[0]);
		if(!RNode.bLeaf)
		{
			WorkingAABB.GrowToInclude(RNode.ChildrenBounds[1]);
		}

		int32 BestSiblingIdx = RootNode;
		FReal BestCost = WorkingAABB.GetArea();
		PriorityQ.Emplace(RNode, RootNode, 0.0f);

		while (PriorityQ.Num() - QIndex)
		{
			// Pop from queue
			const FNodeIndexAndCost NodeAndCost = PriorityQ[QIndex];
			FNode& TestNode = NodeAndCost.template Get<0>();
			const FReal SumDeltaCost = NodeAndCost.template Get<2>();
			QIndex++;

			// TestSibling bounds union with new bounds
			bool bAddToLeaf = false;
			WorkingAABB = TestNode.ChildrenBounds[0];
			if (!TestNode.bLeaf)
			{
				WorkingAABB.GrowToInclude(TestNode.ChildrenBounds[1]);
			}
			else
			{
				int32 LeafIdx = TestNode.ChildrenNodes[0];
				bAddToLeaf = Leaves[LeafIdx].GetElementCount() < FAABBTreeCVars::DynamicTreeLeafCapacity;
			}

			const FReal TestSiblingArea = WorkingAABB.GetArea();

			WorkingAABB.GrowToInclude(NewBounds);

			const FReal NewPotentialNodeArea = WorkingAABB.GetArea();
			const FReal CostForChoosingNode = NewPotentialNodeArea + SumDeltaCost;
			
			if (bAddToLeaf)
			{
				// No new node is added (we can experiment with this cost function
				// It is faster overall if we don't subtract here
				//CostForChoosingNode -= TestSiblingArea;
			}
			
			const FReal NewDeltaCost = NewPotentialNodeArea - TestSiblingArea;
			// Did we get a better cost?
			if (CostForChoosingNode < BestCost)
			{
				BestCost = CostForChoosingNode;
				BestSiblingIdx = NodeAndCost.template Get<1>();
				bOutAddToLeaf = bAddToLeaf;
			}

			// Lower bound of Children costs
			const FReal NewCost = NewDeltaCost + SumDeltaCost;
			const FReal ChildCostLowerBound = NewBoundsArea + NewCost;

			if (!TestNode.bLeaf && ChildCostLowerBound < BestCost)
			{
				// Now we will push the children
				PriorityQ.Reserve(PriorityQ.Num() + 2);
				PriorityQ.Emplace(Nodes[TestNode.ChildrenNodes[0]], TestNode.ChildrenNodes[0], NewCost);
				PriorityQ.Emplace(Nodes[TestNode.ChildrenNodes[1]], TestNode.ChildrenNodes[1], NewCost);
			}

		}

		return BestSiblingIdx;
	}

	// Rotate nodes to decrease tree cost
	// Grandchildren can swap with their aunts
	void RotateNode(uint32 NodeIdx, bool debugAssert = false)
	{
		int32 BestGrandChildToSwap = INDEX_NONE; // GrandChild of NodeIdx
		int32 BestAuntToSwap = INDEX_NONE; // Aunt of BestGrandChildToSwap
		FReal BestDeltaCost = 0.0f; // Negative values are cost reductions, doing nothing changes the cost with 0

		check(!Nodes[NodeIdx].bLeaf);
		// Check both children of NodeIdx
		for (uint32 AuntLocalIdx = 0; AuntLocalIdx < 2; AuntLocalIdx++)
		{
			int32 Aunt = Nodes[NodeIdx].ChildrenNodes[AuntLocalIdx];
			int32 Mother = Nodes[NodeIdx].ChildrenNodes[AuntLocalIdx ^ 1];
			if (Nodes[Mother].bLeaf)
			{
				continue;
			}
			for (int32 GrandChild : Nodes[Mother].ChildrenNodes)
			{
				// Only the Mother's cost will change
				TAABB<T, 3> NewMotherAABB{ Nodes[NodeIdx].ChildrenBounds[AuntLocalIdx]}; // Aunt will be under mother now
				NewMotherAABB.GrowToInclude(Nodes[Mother].ChildrenBounds[GetSiblingIndex(GrandChild)]); // Add the Grandchild's sibling cost
				FReal MotherCostWithoutRotation = Nodes[NodeIdx].ChildrenBounds[AuntLocalIdx ^ 1].GetArea();
				FReal MotherCostWithRotation = NewMotherAABB.GetArea();
				FReal DeltaCost = MotherCostWithRotation - MotherCostWithoutRotation;

				if (DeltaCost < BestDeltaCost)
				{
					BestDeltaCost = DeltaCost;
					BestAuntToSwap = Aunt;
					BestGrandChildToSwap = GrandChild;
				}

			}
		}

		// Now do the rotation if required
		if (BestGrandChildToSwap != INDEX_NONE)
		{
			check(BestAuntToSwap != INDEX_NONE);
			if (debugAssert)
			{
				check(false);
			}

			int32 AuntLocalChildIdx = WhichChildAmI(BestAuntToSwap);
			int32 GrandChildLocalChildIdx = WhichChildAmI(BestGrandChildToSwap);

			int32 MotherOfBestGrandChild = Nodes[BestGrandChildToSwap].ParentNode;

			if (UNLIKELY(!ensure(BestGrandChildToSwap != NodeIdx)))
			{
				UE_LOGF(LogChaos, Warning, "AABBTree: 1: Rotate Node loop detected");
				return;
			}

			if (UNLIKELY(!ensure(BestAuntToSwap != MotherOfBestGrandChild)))
			{
				// This really should not happen, but was due to NaNs entering the bounds (FIXED)
				UE_LOGF(LogChaos, Warning, "AABBTree: 2: Rotate Node loop detected");
				return;
			}

			// Modify NodeIdx 
			Nodes[NodeIdx].ChildrenNodes[AuntLocalChildIdx] = BestGrandChildToSwap;
			
			// Modify BestGrandChildToSwap
			Nodes[BestGrandChildToSwap].ParentNode = NodeIdx;
			
			// Modify BestAuntToSwap
			Nodes[BestAuntToSwap].ParentNode = MotherOfBestGrandChild;
			// Modify MotherOfBestGrandChild
			Nodes[MotherOfBestGrandChild].ChildrenNodes[GrandChildLocalChildIdx] = BestAuntToSwap;
			// Swap the bounds
			TAABB<T, 3> AuntAABB = Nodes[NodeIdx].ChildrenBounds[AuntLocalChildIdx];
			Nodes[NodeIdx].ChildrenBounds[AuntLocalChildIdx] = Nodes[MotherOfBestGrandChild].ChildrenBounds[GrandChildLocalChildIdx];
			Nodes[MotherOfBestGrandChild].ChildrenBounds[GrandChildLocalChildIdx] = AuntAABB;
			// Update the other child bound of NodeIdx
			Nodes[NodeIdx].ChildrenBounds[AuntLocalChildIdx ^ 1] = Nodes[MotherOfBestGrandChild].ChildrenBounds[0];
			Nodes[NodeIdx].ChildrenBounds[AuntLocalChildIdx ^ 1].GrowToInclude(Nodes[MotherOfBestGrandChild].ChildrenBounds[1]);
		}
	}

	// Returns the inserted node and leaf
	NodeAndLeafIndices InsertLeaf(const TPayloadType& Payload, const TAABB<T, 3>& NewBounds)
	{

		// Slow Debug Code
		//if (GetUniqueIdx(Payload).Idx == 5)
		//{
		//	DynamicTreeDebugStats();
		//}

		// Empty tree case
		if(RootNode == INDEX_NONE)
		{
			NodeAndLeafIndices NewIndices = AllocateLeafNodeAndLeaf(Payload, NewBounds);
			RootNode = NewIndices.NodeIdx;
			return NewIndices;
		}

		// Find the best sibling
		bool bAddToLeaf;
		int32 BestSibling = FindBestSibling(NewBounds, bAddToLeaf);

		if (bAddToLeaf)
		{
			const int32 LeafIdx = Nodes[BestSibling].ChildrenNodes[0];
			Leaves[LeafIdx].AddElement(TPayloadBoundsElement<TPayloadType, T>{Payload, NewBounds});
			Leaves[LeafIdx].RecomputeBounds(bDynamicTree);
			TAABB<T, 3> ExpandedBounds = Leaves[LeafIdx].GetBounds();
			ExpandedBounds.Thicken(FAABBTreeCVars::DynamicTreeBoundingBoxPadding);
			Nodes[BestSibling].ChildrenBounds[0] = ExpandedBounds;
			UpdateAncestorBounds(BestSibling, true);
			return NodeAndLeafIndices{ BestSibling , LeafIdx };
		}

		const NodeAndLeafIndices NewLeafIndices = AllocateLeafNodeAndLeaf(Payload, NewBounds);

		// New internal parent node
		const int32 OldParent = Nodes[BestSibling].ParentNode;
		const int32 NewParent = AllocateInternalNode();
		FNode& NewParentNode = Nodes[NewParent];
		if (UNLIKELY(!ensure(NewParent != OldParent)))
		{
			UE_LOGF(LogChaos, Warning, "AABBTree: 1: Insert leaf loop detected");
		}
		NewParentNode.ParentNode = OldParent;
		NewParentNode.ChildrenNodes[0] = BestSibling;
		NewParentNode.ChildrenNodes[1] = NewLeafIndices.NodeIdx;
		NewParentNode.ChildrenBounds[0] = Nodes[BestSibling].ChildrenBounds[0];
		if (!Nodes[BestSibling].bLeaf)
		{
			NewParentNode.ChildrenBounds[0].GrowToInclude(Nodes[BestSibling].ChildrenBounds[1]);
		}
		NewParentNode.ChildrenBounds[1] = Nodes[NewLeafIndices.NodeIdx].ChildrenBounds[0];

		if (OldParent != INDEX_NONE)
		{
			const int32 ChildIdx = WhichChildAmI(BestSibling);
			Nodes[OldParent].ChildrenNodes[ChildIdx] = NewParent;
		}
		else
		{
			RootNode = NewParent;
		}

		if (UNLIKELY(!ensure(BestSibling != NewParent)))
		{
			UE_LOGF(LogChaos, Warning, "AABBTree: 2: Insert leaf loop detected");
		}
		Nodes[BestSibling].ParentNode = NewParent;
		if (UNLIKELY(!ensure(NewLeafIndices.NodeIdx != NewParent)))
		{
			UE_LOGF(LogChaos, Warning, "AABBTree: 3: Insert leaf loop detected");
		}
		Nodes[NewLeafIndices.NodeIdx].ParentNode = NewParent;

		UpdateAncestorBounds(NewParent, true);

		return NewLeafIndices;
	}	

	void  UpdateAncestorBounds(int32 NodeIdx, bool bDoRotation = false)
	{
		int32 CurrentNodeIdx = NodeIdx;
		int32 ParentNodeIdx = Nodes[NodeIdx].ParentNode;


		// This should not be required
		/*if (bDoRotation && NodeIdx != INDEX_NONE)
		{
			RotateNode(NodeIdx,true); 
		}*/
		while (ParentNodeIdx != INDEX_NONE)
		{
			if (UNLIKELY(!ensure(CurrentNodeIdx != ParentNodeIdx)))
			{
				UE_LOGF(LogChaos, Warning, "AABBTree: 1: UpdateAncestorBounds loop detected");
				check(false); // Crash here, this is not recoverable
				return;
			}
			int32 ChildIndex = WhichChildAmI(CurrentNodeIdx);
			Nodes[ParentNodeIdx].ChildrenBounds[ChildIndex] = Nodes[CurrentNodeIdx].ChildrenBounds[0];
			if (!Nodes[CurrentNodeIdx].bLeaf)
			{
				Nodes[ParentNodeIdx].ChildrenBounds[ChildIndex].GrowToInclude(Nodes[CurrentNodeIdx].ChildrenBounds[1]);
			}

			if (bDoRotation)
			{
				RotateNode(ParentNodeIdx);
			}

			CurrentNodeIdx = ParentNodeIdx;
			ParentNodeIdx = Nodes[CurrentNodeIdx].ParentNode;
		}
	}

	void RemoveLeafNode(int32 LeafNodeIdx, const TPayloadType& Payload)
	{
		if (UNLIKELY(!ensure(Nodes[LeafNodeIdx].bLeaf == true)))
		{
			UE_LOGF(LogChaos, Warning, "AABBTree: RemoveLeafNode on node that is not a leaf");
		}


		int32 LeafIdx = Nodes[LeafNodeIdx].ChildrenNodes[0];

		if (Leaves[LeafIdx].GetElementCount() > 1)
		{
			Leaves[LeafIdx].RemoveElement(Payload);
			Leaves[LeafIdx].RecomputeBounds(bDynamicTree);
			TAABB<T, 3> ExpandedBounds = Leaves[LeafIdx].GetBounds();
			ExpandedBounds.Thicken(FAABBTreeCVars::DynamicTreeBoundingBoxPadding);
			Nodes[LeafNodeIdx].ChildrenBounds[0] = ExpandedBounds;
			UpdateAncestorBounds(LeafNodeIdx);
			return;
		}
		else
		{
			Leaves[LeafIdx].RemoveElement(Payload); // Just to check if the element was in there to begin with
		}

		int32 ParentNodeIdx = Nodes[LeafNodeIdx].ParentNode;

		if (ParentNodeIdx != INDEX_NONE)
		{
			int32 GrandParentNodeIdx = Nodes[ParentNodeIdx].ParentNode;
			int32 SiblingNodeLocalIdx = GetSiblingIndex(LeafNodeIdx);
			int32 SiblingNodeIdx = Nodes[ParentNodeIdx].ChildrenNodes[SiblingNodeLocalIdx];

			if (GrandParentNodeIdx != INDEX_NONE)
			{
				int32 ChildLocalIdx = WhichChildAmI(ParentNodeIdx);
				Nodes[GrandParentNodeIdx].ChildrenNodes[ChildLocalIdx] = SiblingNodeIdx;
			}
			else
			{
				RootNode = SiblingNodeIdx;
			}

			if (UNLIKELY(!ensure(SiblingNodeIdx != GrandParentNodeIdx)))
			{
				UE_LOGF(LogChaos, Warning, "AABBTree: RemoveLeafNode loop detected");
			}

			Nodes[SiblingNodeIdx].ParentNode = GrandParentNodeIdx;
			UpdateAncestorBounds(SiblingNodeIdx);
			DeAllocateInternalNode(ParentNodeIdx);
		}
		else
		{
			RootNode = INDEX_NONE;
		}
		DeAllocateLeafNode(LeafNodeIdx);
	}

	virtual bool RemoveElement(const TPayloadType& Payload) override
	{
		if (UNLIKELY(!ensure(bModifyingTreeMultiThreadingFastCheck == false)))
		{
			UE_LOGF(LogChaos, Warning, "AABBTree: RemoveElement unsafe updated from multiple threads detected");
		}
		bModifyingTreeMultiThreadingFastCheck = true;
		if (ensure(bMutable))
		{
			if (FAABBTreePayloadInfo* PayloadInfo = PayloadToInfo.Find(Payload))
			{
				if (PayloadInfo->GlobalPayloadIdx != INDEX_NONE)
				{
					ensure(PayloadInfo->DirtyPayloadIdx == INDEX_NONE);
					ensure(PayloadInfo->DirtyGridOverflowIdx == INDEX_NONE);
					ensure(PayloadInfo->LeafIdx == INDEX_NONE);
					if (PayloadInfo->GlobalPayloadIdx + 1 < GlobalPayloads.Num())
					{
						auto LastGlobalPayload = GlobalPayloads.Last().Payload;
						PayloadToInfo.FindChecked(LastGlobalPayload).GlobalPayloadIdx = PayloadInfo->GlobalPayloadIdx;
					}
					GlobalPayloads.RemoveAtSwap(PayloadInfo->GlobalPayloadIdx);
				}
				else if (PayloadInfo->DirtyPayloadIdx != INDEX_NONE)
				{
					if (DirtyElementTree != nullptr)
					{
						DirtyElementTree->RemoveLeafNode(PayloadInfo->DirtyPayloadIdx, Payload);
					}
					else if (DirtyElementGridEnabled())
					{
						DeleteDirtyParticleEverywhere(PayloadInfo->DirtyPayloadIdx, PayloadInfo->DirtyGridOverflowIdx);
					}
					else
					{
						if (PayloadInfo->DirtyPayloadIdx + 1 < DirtyElements.Num())
						{
							auto LastDirtyPayload = DirtyElements.Last().Payload;
							PayloadToInfo.FindChecked(LastDirtyPayload).DirtyPayloadIdx = PayloadInfo->DirtyPayloadIdx;
						}
						DirtyElements.RemoveAtSwap(PayloadInfo->DirtyPayloadIdx);
					}
				}
				else if (ensure(PayloadInfo->LeafIdx != INDEX_NONE))
				{
					if (bDynamicTree)
					{
						RemoveLeafNode(PayloadInfo->NodeIdx, Payload);
					}
					else
					{
						Leaves[PayloadInfo->LeafIdx].RemoveElement(Payload);
					}
				}

				PayloadToInfo.Remove(Payload);
				bShouldRebuild = true;
				bModifyingTreeMultiThreadingFastCheck = false;
				return true;
			}
		}
		bModifyingTreeMultiThreadingFastCheck = false;
		return false;
	}

	// Returns true if the payload's leaf need to be updated, false if the Payload can stay in the same leaf without changing the leaf size.
	// In the case this function returns false, the payload's bounding volume will be updated. 
	virtual bool NeedUpdateElement(const TPayloadType& Payload, const TAABB<T, 3>& NewBounds) override
	{
		ensure(bMutable);
		FAABBTreePayloadInfo* PayloadInfo = PayloadToInfo.Find(Payload);
		if (PayloadInfo && bDynamicTree)
		{
			if (PayloadInfo->LeafIdx != INDEX_NONE)
			{
				// The leaf node bounds can be larger than the actual leave bound
				const TAABB<T, 3>& LeafNodeBounds = Nodes[PayloadInfo->NodeIdx].ChildrenBounds[0];
				if (LeafNodeBounds.Contains(NewBounds.Min()) && LeafNodeBounds.Contains(NewBounds.Max()))
				{
					// We still need to update the constituent bounds, but the leaf won't change
					Leaves[PayloadInfo->LeafIdx].UpdateElementWithoutDirty(Payload, NewBounds);
					return false;
				}
			}
		}
		return true;
	}

	// Returns true if element was updated, or false when it was added instead
	virtual bool UpdateElement(const TPayloadType& Payload, const TAABB<T, 3>& NewBounds, bool bInHasBounds) override
	{
		if (UNLIKELY(!ensure(bModifyingTreeMultiThreadingFastCheck == false)))
		{
			UE_LOGF(LogChaos, Warning, "AABBTree: UpdateElement unsafe updated from multiple threads detected");
		}
		bModifyingTreeMultiThreadingFastCheck = true;
#if !WITH_EDITOR
		//CSV_SCOPED_TIMING_STAT(ChaosPhysicsTimers, AABBTreeUpdateElement)
		//CSV_CUSTOM_STAT(ChaosPhysicsTimers, 1, 1, ECsvCustomStatOp::Accumulate);
		//CSV_CUSTOM_STAT(PhysicsCounters, NumIntoNP, 1, ECsvCustomStatOp::Accumulate);
#endif

		bool bHasBounds = bInHasBounds;
		// If bounds are bad, use global
		if (bHasBounds && ValidateBounds(NewBounds) == false)
		{
			bHasBounds = false;
			TSharedPtr<FString, ESPMode::ThreadSafe> DebugName;
#if CHAOS_DEBUG_NAME			
			if constexpr (std::is_same_v<TPayloadType, FAccelerationStructureHandle>)
			{
				if (IsInPhysicsThreadContext())
				{
					if (const FGeometryParticleHandle* Particle = Payload.GetGeometryParticleHandle_PhysicsThread())
					{
						DebugName = Particle->DebugName();
					}
				}
				else
				{
					if (const FGeometryParticle* Particle = Payload.GetExternalGeometryParticle_ExternalThread())
					{
						DebugName = Particle->DebugName();
					}
				}
			}
#endif
			UE_LOGF(LogChaos, Warning, "AABBTree encountered invalid bounds. Min: %ls Max: %ls Name: %ls"
				, *NewBounds.Min().ToString() , *NewBounds.Max().ToString()
				, DebugName.IsValid() ? **DebugName : TEXT("<Unknown>"));
		}

		bool bElementExisted = true;
		if (ensure(bMutable))
		{
			FAABBTreePayloadInfo* PayloadInfo = PayloadToInfo.Find(Payload);
			if (PayloadInfo)
			{
				if (PayloadInfo->LeafIdx != INDEX_NONE)
				{
					//If we are still within the same leaf bounds, do nothing, don't detect a change either
					if (bHasBounds) 
					{
						if (bDynamicTree)
						{
							// The leaf node bounds can be larger than the actual leave bound
							const TAABB<T, 3>& LeafNodeBounds = Nodes[PayloadInfo->NodeIdx].ChildrenBounds[0];
							if (LeafNodeBounds.Contains(NewBounds.Min()) && LeafNodeBounds.Contains(NewBounds.Max()))
							{
								// We still need to update the constituent bounds
								Leaves[PayloadInfo->LeafIdx].UpdateElement(Payload, NewBounds, bHasBounds);
								bModifyingTreeMultiThreadingFastCheck = false;
								return bElementExisted;
							}
						}
						else
						{
							const TAABB<T, 3>& LeafBounds = Leaves[PayloadInfo->LeafIdx].GetBounds();
							if (LeafBounds.Contains(NewBounds.Min()) && LeafBounds.Contains(NewBounds.Max()))
							{
								// We still need to update the constituent bounds
								Leaves[PayloadInfo->LeafIdx].UpdateElement(Payload, NewBounds, bHasBounds);
								bModifyingTreeMultiThreadingFastCheck = false;
								return bElementExisted;
							}
						}
					}

					// DBVH remove from tree

					if (bDynamicTree)
					{
						RemoveLeafNode(PayloadInfo->NodeIdx, Payload);
						PayloadInfo->LeafIdx = INDEX_NONE;
						PayloadInfo->NodeIdx = INDEX_NONE;
					}
					else
					{
						Leaves[PayloadInfo->LeafIdx].RemoveElement(Payload);
						PayloadInfo->LeafIdx = INDEX_NONE;
					}
				}
			}
			else
			{
				bElementExisted = false;
				PayloadInfo = &PayloadToInfo.Add(Payload);
			}

			bShouldRebuild = true;

			bool bTooBig = false;
			if (bHasBounds)
			{
				if (NewBounds.Extents().Max() > MaxPayloadBounds)
				{
					bTooBig = true;
					bHasBounds = false;
				}
			}

			if (bHasBounds)
			{
				if (PayloadInfo->DirtyPayloadIdx == INDEX_NONE)
				{
					if (bDynamicTree)
					{
						NodeAndLeafIndices Indices = InsertLeaf(Payload, NewBounds);
						PayloadInfo->NodeIdx = Indices.NodeIdx;
						PayloadInfo->LeafIdx = Indices.LeafIdx;
					}
					else
					{
						if (DirtyElementTree)
						{
							// We save the node index for the dirty tree here:
							PayloadInfo->DirtyPayloadIdx = DirtyElementTree->InsertLeaf(Payload, NewBounds).NodeIdx;
						}
						else
						{
							PayloadInfo->DirtyPayloadIdx = DirtyElements.Add(FElement{ Payload, NewBounds });

							if (DirtyElementGridEnabled())
							{
								PayloadInfo->DirtyGridOverflowIdx = AddDirtyElementToGrid(NewBounds, PayloadInfo->DirtyPayloadIdx);
							}
						}						
					}
				}
				else
				{
					const int32 DirtyElementIndex = PayloadInfo->DirtyPayloadIdx;
					if (DirtyElementTree)
					{
						// The leaf node bounds can be larger than the actual leave bound
						const TAABB<T, 3>& LeafNodeBounds = DirtyElementTree->Nodes[DirtyElementIndex].ChildrenBounds[0];
						if (LeafNodeBounds.Contains(NewBounds.Min()) && LeafNodeBounds.Contains(NewBounds.Max()))
						{
							// We still need to update the constituent bounds
							const int32 DirtyLeafIndex = DirtyElementTree->Nodes[DirtyElementIndex].ChildrenNodes[0]; // A Leaf node's first child is the leaf index
							DirtyElementTree->Leaves[DirtyLeafIndex].UpdateElement(Payload, NewBounds, bHasBounds);
							DirtyElementTree->Leaves[DirtyLeafIndex].RecomputeBounds(bDynamicTree);
						}
						else
						{
							// Reinsert the leaf
							DirtyElementTree->RemoveLeafNode(DirtyElementIndex, Payload);
							PayloadInfo->DirtyPayloadIdx = DirtyElementTree->InsertLeaf(Payload, NewBounds).NodeIdx;
						}
					}
					else
					{
						if (DirtyElementGridEnabled())
						{
							//CSV_SCOPED_TIMING_STAT(ChaosPhysicsTimers, AABBUpElement)
							PayloadInfo->DirtyGridOverflowIdx = UpdateDirtyElementInGrid(NewBounds, DirtyElementIndex, PayloadInfo->DirtyGridOverflowIdx);
						}
						DirtyElements[DirtyElementIndex].Bounds = NewBounds;
						UpdateElementHelper(DirtyElements[DirtyElementIndex].Payload, Payload);
					}				
				}

				// Handle something that previously did not have bounds that may be in global elements.
				if (PayloadInfo->GlobalPayloadIdx != INDEX_NONE)
				{
					if (PayloadInfo->GlobalPayloadIdx + 1 < GlobalPayloads.Num())
					{
						auto LastGlobalPayload = GlobalPayloads.Last().Payload;
						PayloadToInfo.FindChecked(LastGlobalPayload).GlobalPayloadIdx = PayloadInfo->GlobalPayloadIdx;
					}
					GlobalPayloads.RemoveAtSwap(PayloadInfo->GlobalPayloadIdx);

					PayloadInfo->GlobalPayloadIdx = INDEX_NONE;
				}
			}
			else
			{
				TAABB<T, 3> GlobalBounds = bTooBig ? NewBounds : TAABB<T, 3>(TVec3<T>(TNumericLimits<T>::Lowest()), TVec3<T>(TNumericLimits<T>::Max()));
				if (PayloadInfo->GlobalPayloadIdx == INDEX_NONE)
				{
					PayloadInfo->GlobalPayloadIdx = GlobalPayloads.Add(FElement{ Payload, GlobalBounds });
				}
				else
				{
					GlobalPayloads[PayloadInfo->GlobalPayloadIdx].Bounds = GlobalBounds;
					UpdateElementHelper(GlobalPayloads[PayloadInfo->GlobalPayloadIdx].Payload, Payload);
				}

				// Handle something that previously had bounds that may be in dirty elements.
				if (PayloadInfo->DirtyPayloadIdx != INDEX_NONE)
				{
					if (DirtyElementTree)
					{
						DirtyElementTree->RemoveLeafNode(PayloadInfo->DirtyPayloadIdx, Payload);
					}
					else
					{
						if (DirtyElementGridEnabled())
						{
							DeleteDirtyParticleEverywhere(PayloadInfo->DirtyPayloadIdx, PayloadInfo->DirtyGridOverflowIdx);
						}
						else
						{
							if (PayloadInfo->DirtyPayloadIdx + 1 < DirtyElements.Num())
							{
								auto LastDirtyPayload = DirtyElements.Last().Payload;
								PayloadToInfo.FindChecked(LastDirtyPayload).DirtyPayloadIdx = PayloadInfo->DirtyPayloadIdx;
							}
							DirtyElements.RemoveAtSwap(PayloadInfo->DirtyPayloadIdx);
						}
					}

					PayloadInfo->DirtyPayloadIdx = INDEX_NONE;
					PayloadInfo->DirtyGridOverflowIdx = INDEX_NONE;
				}
			}
		}

		if(!DirtyElementTree && !bDynamicTree && DirtyElements.Num() > MaxDirtyElements)
		{
			UE_LOGF(LogChaos, Verbose, "Bounding volume exceeded maximum dirty elements (%d dirty of max %d) and is forcing a tree rebuild.", DirtyElements.Num(), MaxDirtyElements);
			ReoptimizeTree();
		}
		bModifyingTreeMultiThreadingFastCheck = false;
		return bElementExisted;
	}

	int32 NumDirtyElements() const
	{
		return DirtyElements.Num();
	}

	// Some useful statistics
	const AABBTreeStatistics& GetAABBTreeStatistics()
	{
		// Update the stats that needs it first
		TreeStats.StatNumDirtyElements = DirtyElements.Num();
		TreeStats.StatNumGridOverflowElements = DirtyElementsGridOverflow.Num();
		return TreeStats;
	}

	const AABBTreeExpensiveStatistics& GetAABBTreeExpensiveStatistics()
	{
		TreeExpensiveStats.StatMaxDirtyElements = DirtyElements.Num();
		TreeExpensiveStats.StatMaxNumLeaves = Leaves.Num();
		int32 StatMaxLeafSize = 0;
		for (int LeafIndex = 0; LeafIndex < Leaves.Num(); LeafIndex++)
		{
			const TLeafType& Leaf = Leaves[LeafIndex];

			StatMaxLeafSize = FMath::Max(StatMaxLeafSize, (int32)Leaf.GetElementCount());
		}
		TreeExpensiveStats.StatMaxLeafSize = StatMaxLeafSize;
		TreeExpensiveStats.StatMaxTreeDepth = (Nodes.Num() == 0) ? 0 : GetSubtreeDepth(0);
		TreeExpensiveStats.StatGlobalPayloadsSize = GlobalPayloads.Num();

		return TreeExpensiveStats;
	}

	const int32 GetSubtreeDepth(const int32 NodeIdx)
	{
		const FNode& Node = Nodes[NodeIdx];
		if (Node.bLeaf)
		{
			return 1;
		}
		else
		{
			return FMath::Max(GetSubtreeDepth(Node.ChildrenNodes[0]), GetSubtreeDepth(Node.ChildrenNodes[1])) + 1;
		}
	}

	const TArray<TPayloadBoundsElement<TPayloadType, T>>& GlobalObjects() const
	{
		return GlobalPayloads;
	}


	virtual bool ShouldRebuild() override { return bDynamicTree ? false : bShouldRebuild; }  // Used to find out if something changed since last reset for optimizations
	// Contract: bShouldRebuild can only ever be cleared by calling the ClearShouldRebuild method, it can be set at will though
	virtual void ClearShouldRebuild() override { bShouldRebuild = false; }

	virtual bool IsTreeDynamic() const override { return bDynamicTree; }
	void SetTreeToDynamic() { bDynamicTree = true; } // Tree cannot be changed back to static for now

	virtual void PrepareCopyTimeSliced(const  ISpatialAcceleration<TPayloadType, T, 3>& InFrom) override
	{
		check(this != &InFrom);
		check(InFrom.GetType() == ESpatialAcceleration::AABBTree);
		const TAABBTree& From = static_cast<const TAABBTree&>(InFrom);

		Reset();

		// Copy all the small objects first

		ISpatialAcceleration<TPayloadType, T, 3>::DeepAssign(From);

		DirtyElementGridCellSize = From.DirtyElementGridCellSize;
		DirtyElementGridCellSizeInv = From.DirtyElementGridCellSizeInv;
		DirtyElementMaxGridCellQueryCount = From.DirtyElementMaxGridCellQueryCount;
		DirtyElementMaxPhysicalSizeInCells = From.DirtyElementMaxPhysicalSizeInCells;
		DirtyElementMaxCellCapacity = From.DirtyElementMaxCellCapacity;

		MaxChildrenInLeaf = From.MaxChildrenInLeaf;
		MaxTreeDepth = From.MaxTreeDepth;
		MaxPayloadBounds = From.MaxPayloadBounds;
		MaxNumToProcess = From.MaxNumToProcess;
		NumProcessedThisSlice = From.NumProcessedThisSlice;

		StartSliceTimeStamp = From.StartSliceTimeStamp;

		bShouldRebuild = From.bShouldRebuild;

		RootNode = From.RootNode;
		FirstFreeInternalNode = From.FirstFreeInternalNode;
		FirstFreeLeafNode = From.FirstFreeLeafNode;

		// Reserve sizes for arrays etc

		Nodes.Reserve(From.Nodes.Num());
		Leaves.Reserve(From.Leaves.Num());
		DirtyElements.Reserve(From.DirtyElements.Num());
		CellHashToFlatArray.Reserve(From.CellHashToFlatArray.Num());
		FlattenedCellArrayOfDirtyIndices.Reserve(From.FlattenedCellArrayOfDirtyIndices.Num());
		DirtyElementsGridOverflow.Reserve(From.DirtyElementsGridOverflow.Num());
		GlobalPayloads.Reserve(From.GlobalPayloads.Num());
		PayloadToInfo.Reserve(From.PayloadToInfo.Num());
		OverlappingLeaves.Reserve(From.OverlappingLeaves.Num());
		OverlappingOffsets.Reserve(From.OverlappingOffsets.Num());
		OverlappingPairs.Reserve(From.OverlappingPairs.Num());
		OverlappingCounts.Reserve(From.OverlappingCounts.Num());

		this->SetAsyncTimeSlicingComplete(false);

		if (From.DirtyElementTree)
		{
			if (!DirtyElementTree)
			{
				DirtyElementTree = TUniquePtr<TAABBTree>(new TAABBTree());
			}
			DirtyElementTree->PrepareCopyTimeSliced(*(From.DirtyElementTree));
		}
		else
		{
			DirtyElementTree = nullptr;
		}
	}

	virtual void ProgressCopyTimeSliced(const ISpatialAcceleration<TPayloadType, T, 3>& InFrom, int MaximumBytesToCopy) override
	{
		check(this != &InFrom);
		check(InFrom.GetType() == ESpatialAcceleration::AABBTree);
		const TAABBTree& From = static_cast<const TAABBTree&>(InFrom);

		int32 SizeToCopyLeft = MaximumBytesToCopy;
		check(From.CellHashToFlatArray.Num() == 0); // Partial Copy of TMAPs not implemented, and this should be empty for our current use cases

		auto CanContinueCopyingDataCallback = [this](int32 MaxSizeToCopy, int32 CurrentCopiedSize)
		{
			bool bCanContinueCopy = true;
			const bool bForceCopyAll = MaxSizeToCopy == -1;
			if (bForceCopyAll)
			{
				return bCanContinueCopy;
			}

			if (FAABBTimeSliceCVars::bUseTimeSliceMillisecondBudget)
			{
				// Checking for platform time every time is expensive as its cost adds up fast, so only do it between a configured amount of elements.
				// This means we might overshoot the budget, but on the other hand we will keep most of the time doing actual work.
				CurrentDataElementsCopiedSinceLastCheck++;
				if (CurrentDataElementsCopiedSinceLastCheck > FAABBTimeSliceCVars::MinDataChunkToProcessBetweenTimeChecks)
				{
					CurrentDataElementsCopiedSinceLastCheck = 0;
					return bCanContinueCopy;
				}

				const double ElapseTime = FPlatformTime::Seconds() - StartSliceTimeStamp;

				if (!FMath::IsNearlyZero(StartSliceTimeStamp) && ElapseTime > FAABBTimeSliceCVars::MaxProcessingTimePerSliceSeconds)
				{
					bCanContinueCopy = false;
				}
			}
			else
			{
				bCanContinueCopy = MaxSizeToCopy < 0 || CurrentCopiedSize < MaxSizeToCopy;
			}

			return bCanContinueCopy;
		};

		constexpr int32 SizeCopiedSoFar = 0;
		if (!CanContinueCopyingDataCallback(MaximumBytesToCopy, SizeCopiedSoFar))
		{
			// For data copy, the time stamp is set from the setup phase, and it is copied from the tree we are copying from.
			// Therefore if we reach this point with no time available left, just reset it so the next frame can start counting from scratch 
			StartSliceTimeStamp = 0.0;
			return;
		}

		if (FMath::IsNearlyZero(StartSliceTimeStamp))
		{
			StartSliceTimeStamp = FPlatformTime::Seconds();
		}

		if (!ContinueTimeSliceCopy(From.Nodes, Nodes, SizeToCopyLeft, CanContinueCopyingDataCallback))
		{
			return;
		}
		if (!ContinueTimeSliceCopy(From.Leaves, Leaves, SizeToCopyLeft, CanContinueCopyingDataCallback))
		{
			return;
		}
		if (!ContinueTimeSliceCopy(From.DirtyElements, DirtyElements, SizeToCopyLeft, CanContinueCopyingDataCallback))
		{
			return;
		}

		if (!ContinueTimeSliceCopy(From.FlattenedCellArrayOfDirtyIndices, FlattenedCellArrayOfDirtyIndices, SizeToCopyLeft, CanContinueCopyingDataCallback))
		{
			return;
		}
		if (!ContinueTimeSliceCopy(From.DirtyElementsGridOverflow, DirtyElementsGridOverflow, SizeToCopyLeft, CanContinueCopyingDataCallback))
		{
			return;
		}
		if (!ContinueTimeSliceCopy(From.GlobalPayloads, GlobalPayloads, SizeToCopyLeft, CanContinueCopyingDataCallback))
		{
			return;
		}
		if (!ContinueTimeSliceCopy(From.PayloadToInfo, PayloadToInfo, SizeToCopyLeft, CanContinueCopyingDataCallback))
		{
			return;
		}
		if (!ContinueTimeSliceCopy(From.OverlappingLeaves, OverlappingLeaves, SizeToCopyLeft, CanContinueCopyingDataCallback))
		{
			return;
		}
		if (!ContinueTimeSliceCopy(From.OverlappingOffsets, OverlappingOffsets, SizeToCopyLeft, CanContinueCopyingDataCallback))
		{
			return;
		}
		if (!ContinueTimeSliceCopy(From.OverlappingCounts, OverlappingCounts, SizeToCopyLeft, CanContinueCopyingDataCallback))
		{
			return;
		}
		if (!ContinueTimeSliceCopy(From.OverlappingPairs, OverlappingPairs, SizeToCopyLeft, CanContinueCopyingDataCallback))
		{
			return;
		}

		if (DirtyElementTree)
		{
			check(From.DirtyElementTree);
			DirtyElementTree->ProgressCopyTimeSliced(*(From.DirtyElementTree), MaximumBytesToCopy);
			if (!DirtyElementTree->IsAsyncTimeSlicingComplete())
			{
				return;
			}
		}

		this->SetAsyncTimeSlicingComplete(true);
	}

	// Returns true if bounds appear valid. Returns false if extremely large values, contains NaN, or is empty.
	FORCEINLINE_DEBUGGABLE bool ValidateBounds(const TAABB<T, 3>& Bounds)
	{
		return Bounds.IsValid();
	}

	virtual void Serialize(FChaosArchive& Ar) override
	{
		Ar.UsingCustomVersion(FExternalPhysicsCustomObjectVersion::GUID);

		if (Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) < FExternalPhysicsCustomObjectVersion::RemovedAABBTreeFullBounds)
		{
			// Serialize out unused aabb for earlier versions
			TAABB<T, 3> Dummy(TVec3<T>((T)0), TVec3<T>((T)0));
			TBox<T, 3>::SerializeAsAABB(Ar, Dummy);
		}
		Ar << Nodes;
		Ar << Leaves;
		Ar << DirtyElements;
		Ar << GlobalPayloads;

		bool bSerializePayloadToInfo = !bMutable;
		if (Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) >= FExternalPhysicsCustomObjectVersion::ImmutableAABBTree)
		{
			Ar << bSerializePayloadToInfo;
		}
		else
		{
			bSerializePayloadToInfo = true;
		}

		if (bSerializePayloadToInfo)
		{
			Ar << PayloadToInfo;

			if (!bMutable)	//if immutable empty this even if we had to serialize it in for backwards compat
			{
				PayloadToInfo.Empty();
			}
		}

		Ar << MaxChildrenInLeaf;
		Ar << MaxTreeDepth;
		Ar << MaxPayloadBounds;

		if (Ar.IsLoading())
		{
			// Disable the Grid until it is rebuilt
			DirtyElementGridCellSize = 0.0f;
			DirtyElementGridCellSizeInv = 1.0f;
			bShouldRebuild = true;
		}

		// Dynamic trees are not serialized/deserialized for now
		if (Ar.IsLoading())
		{
			bModifyingTreeMultiThreadingFastCheck = false;
			bDynamicTree = false;
			bBuildOverlapCache = true;			
			RootNode = INDEX_NONE;
			FirstFreeInternalNode = INDEX_NONE;
			FirstFreeLeafNode = INDEX_NONE;
		}
		else
		{
			ensure(bDynamicTree == false);
		}
	}
	
	/** Given a first node and a leaf index find the overlapping leaves and update the node stack 
	 * @param FirstNode First node to be added to the stack
	 * @param LeafIndex Leaf index for which we want to find the overlapping leaves
	 * @param Nodestack Node stack that will be used to traverse the tree and find the overlapping leaves
	 */
	void FindOverlappingLeaf(const int32 FirstNode, const int32 LeafIndex, TArray<int32>& NodeStack) 
	{
		const TAABB<T,3>& LeafBounds = Leaves[LeafIndex].GetBounds();
				
		NodeStack.Reset();
		NodeStack.Add(FirstNode);
		
		int32 NodeIndex = INDEX_NONE;
		while (NodeStack.Num())
		{
			NodeIndex = NodeStack.Pop(EAllowShrinking::No);
			const FNode& Node = Nodes[NodeIndex];
			
			// If a leaf directly test the bounds
			if (Node.bLeaf)
			{
				if (LeafBounds.Intersects(Leaves[Node.ChildrenNodes[0]].GetBounds()))
				{
					OverlappingLeaves.Add(Node.ChildrenNodes[0]);
				}
			}
			else
			{
				// If not loop over all the children nodes to check if they intersect the bounds
				for(int32 ChildIndex = 0; ChildIndex < 2; ++ChildIndex)
				{
					if(LeafBounds.Intersects(Node.ChildrenBounds[ChildIndex]) && Node.ChildrenNodes[ChildIndex] != INDEX_NONE)
					{
						NodeStack.Add(Node.ChildrenNodes[ChildIndex]);
					}
				}
			}
		}
	}
	
	/** Recursively add overlapping leaves given 2 nodes in the tree */
    void AddNodesOverlappingLeaves(const TAABBTreeNode<T>& LeftNode, const TAABB<T, 3>& LeftBounds,
								  const TAABBTreeNode<T>& RightNode, const TAABB<T, 3>& RightBounds, const bool bDirtyFilter)
	{
		// If dirty filter enabled only look for overlapping leaves if one of the 2 nodes are dirty 
		if(!bDirtyFilter || (bDirtyFilter && (LeftNode.bDirtyNode || RightNode.bDirtyNode)))
		{
			if(LeftBounds.Intersects(RightBounds))
			{
				// If left and right are leaves check for intersection
				if(LeftNode.bLeaf && RightNode.bLeaf)
				{
					const int32 LeftLeaf = LeftNode.ChildrenNodes[0];
					const int32 RightLeaf = RightNode.ChildrenNodes[0];

					// Same condition as for the nodes
					if(!bDirtyFilter || (bDirtyFilter && (Leaves[LeftLeaf].IsLeafDirty() || Leaves[RightLeaf].IsLeafDirty())))
					{
						if(Leaves[LeftLeaf].GetBounds().Intersects(Leaves[RightLeaf].GetBounds()))
						{
							OverlappingPairs.Add(FIntVector2(LeftLeaf, RightLeaf));
							++OverlappingCounts[LeftLeaf];
							++OverlappingCounts[RightLeaf];
						}
					}
				}
				// If only left is a leaf continue recursion with the right node children
				else if(LeftNode.bLeaf)
				{
					AddNodesOverlappingLeaves(LeftNode, LeftBounds, Nodes[RightNode.ChildrenNodes[0]], RightNode.ChildrenBounds[0], bDirtyFilter);
					AddNodesOverlappingLeaves(LeftNode, LeftBounds, Nodes[RightNode.ChildrenNodes[1]], RightNode.ChildrenBounds[1], bDirtyFilter);
				}
				// Otherwise continue recursion with the left node children
				else
				{
					AddNodesOverlappingLeaves(Nodes[LeftNode.ChildrenNodes[0]], LeftNode.ChildrenBounds[0], RightNode, RightBounds, bDirtyFilter);
					AddNodesOverlappingLeaves(Nodes[LeftNode.ChildrenNodes[1]], LeftNode.ChildrenBounds[1], RightNode, RightBounds, bDirtyFilter);
				}
			}
		}
	}
	
	/** Recursively add overlapping leaves given a root node in the tree */
	void AddRootOverlappingLeaves(const TAABBTreeNode<T>& TreeNode, const bool bDirtyFilter)
	{
		if(!TreeNode.bLeaf)
		{
			// Find overlapping leaves within the left and right children
			AddRootOverlappingLeaves(Nodes[TreeNode.ChildrenNodes[0]], bDirtyFilter);
			AddRootOverlappingLeaves(Nodes[TreeNode.ChildrenNodes[1]], bDirtyFilter);

			// Then try finding some overlaps in between the 2 children
			AddNodesOverlappingLeaves(Nodes[TreeNode.ChildrenNodes[0]], TreeNode.ChildrenBounds[0],
									  Nodes[TreeNode.ChildrenNodes[1]], TreeNode.ChildrenBounds[1], bDirtyFilter);
		}
	}

	/** Fill the overlapping pairs from the previous persistent and not dirty leaves */
	void FillPersistentOverlappingPairs()
	{
		for(int32 LeafIndex = 0, NumLeaves = Leaves.Num(); LeafIndex < NumLeaves; ++LeafIndex)
		{
			int32 NumOverlaps = 0;
			if(!Leaves[LeafIndex].IsLeafDirty())
			{
				for(int32 OverlappingIndex = OverlappingOffsets[LeafIndex]; OverlappingIndex < OverlappingOffsets[LeafIndex+1]; ++OverlappingIndex)
				{
					if(!Leaves[OverlappingLeaves[OverlappingIndex]].IsLeafDirty())
					{
						if(LeafIndex < OverlappingLeaves[OverlappingIndex])
						{
							OverlappingPairs.Add(FIntVector2(LeafIndex, OverlappingLeaves[OverlappingIndex]));
						}
						if(LeafIndex != OverlappingLeaves[OverlappingIndex])
						{
							++NumOverlaps;
						}
					}
				}
			}
			// Make sure the leaf is intersecting itself if several elements per leaf
			OverlappingPairs.Add(FIntVector2(LeafIndex, LeafIndex));
			++NumOverlaps;
			
			OverlappingCounts[LeafIndex] = NumOverlaps;
		}
	}
	/** Propagates the leaves dirty flag up to the root node */
	void PropagateLeavesDirtyFlag()
	{
		for(int32 NodeIndex = 0, NumNodes = Nodes.Num(); NodeIndex < NumNodes; ++NodeIndex)
		{
			if(Nodes[NodeIndex].bLeaf)
			{
				Nodes[NodeIndex].bDirtyNode = Leaves[Nodes[NodeIndex].ChildrenNodes[0]].IsLeafDirty();
				if(Nodes[NodeIndex].bDirtyNode)
				{
					int32 NodeParent = Nodes[NodeIndex].ParentNode;
					while(NodeParent != INDEX_NONE && !Nodes[NodeParent].bDirtyNode)
					{
						Nodes[NodeParent].bDirtyNode = true;
						NodeParent = Nodes[NodeParent].ParentNode;
					}
				}
			}
		}
	}

	/** Simultaneous tree descent to compute the overlapping leaves */
	void ComputeOverlappingCacheFromRoot(const bool bDirtyFilter)
	{
		if(!bDynamicTree || (bDynamicTree && RootNode == INDEX_NONE)) return;
		
		OverlappingOffsets.SetNum(Leaves.Num()+1, EAllowShrinking::No);
		OverlappingCounts.SetNum(Leaves.Num(), EAllowShrinking::No);
		OverlappingPairs.Reset();
		
		if(bDirtyFilter)
		{
			FillPersistentOverlappingPairs();
			PropagateLeavesDirtyFlag();
		}
		else
		{
			for(int32 LeafIndex = 0, NumLeaves = Leaves.Num(); LeafIndex < NumLeaves; ++LeafIndex)
			{
				// Make sure the leaf is intersecting itself if several elements per leaf
				OverlappingPairs.Add(FIntVector2(LeafIndex, LeafIndex));
				OverlappingCounts[LeafIndex] = 1;
			}
		}
		AddRootOverlappingLeaves(Nodes[RootNode], bDirtyFilter);

		OverlappingOffsets[0] = 0;
		for(int32 LeafIndex = 0, NumLeaves = Leaves.Num(); LeafIndex < NumLeaves; ++LeafIndex)
		{
			OverlappingOffsets[LeafIndex+1] = OverlappingOffsets[LeafIndex] + OverlappingCounts[LeafIndex];
			OverlappingCounts[LeafIndex] = OverlappingOffsets[LeafIndex];
		}
		OverlappingLeaves.SetNum(OverlappingOffsets.Last(), EAllowShrinking::No);
		for(auto& OverlappingPair : OverlappingPairs)
		{
			if(OverlappingPair[0] != OverlappingPair[1])
			{
				OverlappingLeaves[OverlappingCounts[OverlappingPair[0]]++] = OverlappingPair[1];
				OverlappingLeaves[OverlappingCounts[OverlappingPair[1]]++] = OverlappingPair[0];
			}
			else
			{
				OverlappingLeaves[OverlappingCounts[OverlappingPair[0]]++] = OverlappingPair[0];
			}
		}
		if(bDirtyFilter)
		{
			for(int32 LeafIndex = 0, NumLeaves = Leaves.Num(); LeafIndex < NumLeaves; ++LeafIndex)
			{
				Leaves[LeafIndex].SetDirtyState(false);
			}
			for(int32 NodeIndex = 0, NumNodes = Nodes.Num(); NodeIndex < NumNodes; ++NodeIndex)
			{
				Nodes[NodeIndex].bDirtyNode = false;
			}
		}
	}

	/** Sequential loop over the leaves to fill the overlapping pairs */
	void ComputeOverlappingCacheFromLeaf()
	{
		OverlappingOffsets.SetNum(Leaves.Num()+1, EAllowShrinking::No);
		OverlappingLeaves.Reset();
		
		if(!bDynamicTree || (bDynamicTree && RootNode == INDEX_NONE)) return;
		
		TArray<int32> NodeStack;
		const int32 FirstNode = bDynamicTree ? RootNode : 0;
		
		for(int32 LeafIndex = 0, NumLeaves = Leaves.Num(); LeafIndex < NumLeaves; ++LeafIndex)
		{ 
			OverlappingOffsets[LeafIndex] = OverlappingLeaves.Num();
			FindOverlappingLeaf(FirstNode, LeafIndex, NodeStack);
		}
		OverlappingOffsets.Last() = OverlappingLeaves.Num();
	}

	/** Cache for each leaves all the overlapping leaves*/
	virtual void CacheOverlappingLeaves() override
	{
		if (!bBuildOverlapCache)
		{
			return;
		}

		// Dev settings to switch easily algorithms
		// Will switch to cvars if the leaf version could be faster 
		const bool bCachingRoot = true;
		const bool bDirtyFilter = false;
		
		if(bCachingRoot)
		{
			ComputeOverlappingCacheFromRoot(bDirtyFilter);
		}
		else
		{
			ComputeOverlappingCacheFromLeaf();
		}
	}

	/** Print the overlapping leaves data structure */
	void PrintOverlappingLeaves()
	{
		UE_LOGF(LogChaos, Log, "Num Leaves = %d", Leaves.Num());
		for(int32 LeafIndex = 0, NumLeaves = Leaves.Num(); LeafIndex < NumLeaves; ++LeafIndex)
		{
			auto& Leaf = Leaves[LeafIndex];
			UE_LOGF(LogChaos, Log, "Overlapping Count[%d] = %d with bounds = %f %f %f | %f %f %f", LeafIndex,
				OverlappingOffsets[LeafIndex+1] - OverlappingOffsets[LeafIndex], Leaf.GetBounds().Min()[0],
				Leaf.GetBounds().Min()[1], Leaf.GetBounds().Min()[2], Leaf.GetBounds().Max()[0], Leaf.GetBounds().Max()[1], Leaf.GetBounds().Max()[2]);
			
			for(int32 OverlappingIndex = OverlappingOffsets[LeafIndex]; OverlappingIndex < OverlappingOffsets[LeafIndex+1]; ++OverlappingIndex)
			{
				UE_LOGF(LogChaos, Log, "Overlapping Leaf[%d] = %d", LeafIndex, OverlappingLeaves[OverlappingIndex]);
			}
		}
	}

	const TArray<TAABBTreeNode<T>>& GetNodes() const { return Nodes; }
	const TArray<TLeafType>& GetLeaves() const { return Leaves; }

private:

	void ReoptimizeTree()
	{
		check(!DirtyElementTree && !bDynamicTree);
		TRACE_CPUPROFILER_EVENT_SCOPE(TAABBTree::ReoptimizeTree);
		TArray<FElement> AllElements;

		int32 ReserveCount = DirtyElements.Num() + GlobalPayloads.Num();
		for (int LeafIndex = 0; LeafIndex < Leaves.Num(); LeafIndex++)
		{
			const TLeafType& Leaf = Leaves[LeafIndex];
			ReserveCount += static_cast<int32>(Leaf.GetReserveCount());
		}

		AllElements.Reserve(ReserveCount);

		AllElements.Append(DirtyElements);
		AllElements.Append(GlobalPayloads);

		for (int LeafIndex = 0; LeafIndex < Leaves.Num(); LeafIndex++)
		{
			TLeafType& Leaf = Leaves[LeafIndex];
			Leaf.GatherElements(AllElements);
		}

		TAABBTree NewTree(AllElements);
		*this = NewTree;
		bShouldRebuild = true; // No changes since last time tree was built
	}

	// Returns true if the query should continue
	// Execute a function for all cells found in a query as well as the overflow 
	template <typename FunctionType>
	bool DoForHitGridCellsAndOverflow(TArray<DirtyGridHashEntry>& HashEntryForOverlappedCells, FunctionType Function) const
	{

		// Now merge and iterate the lists of elements found in the overlapping cells
		bool DoneWithGridElements = false;
		bool DoneWithNonGridElements = false;
		int NonGridElementIter = 0;
		while (!DoneWithGridElements || !DoneWithNonGridElements)
		{
			// Get the next dirty element index

			int32 SmallestDirtyParticleIndex = INT_MAX; // Best dirty particle index to find

			if (!DoneWithGridElements)
			{
				// Find the next smallest index 
				// This will start slowing down if we are overlapping a lot of cells
				DoneWithGridElements = true;
				for (const DirtyGridHashEntry& HashEntry : HashEntryForOverlappedCells)
				{
					int32 Count = HashEntry.Count;
					if (Count > 0)
					{
						int32 DirtyParticleIndex = FlattenedCellArrayOfDirtyIndices[HashEntry.Index];
						if (DirtyParticleIndex < SmallestDirtyParticleIndex)
						{
							SmallestDirtyParticleIndex = DirtyParticleIndex;
							DoneWithGridElements = false;
						}
					}
				}
			}

			// Now skip all elements with the same best index
			if (!DoneWithGridElements)
			{
				for (DirtyGridHashEntry& HashEntry : HashEntryForOverlappedCells)
				{
					int32 Count = HashEntry.Count;
					if (Count > 0)
					{
						int32 DirtyParticleIndex = FlattenedCellArrayOfDirtyIndices[HashEntry.Index];
						if (DirtyParticleIndex == SmallestDirtyParticleIndex)
						{
							++HashEntry.Index; // Increment Index
							--HashEntry.Count; // Decrement count
						}
					}
				}
			}

			DoneWithNonGridElements = NonGridElementIter >= DirtyElementsGridOverflow.Num();
			if (DoneWithGridElements && !DoneWithNonGridElements)
			{
				SmallestDirtyParticleIndex = DirtyElementsGridOverflow[NonGridElementIter];
				++NonGridElementIter;
			}

			// Elements that are in the overflow should not also be in the grid
			ensure(DoneWithGridElements || PayloadToInfo.Find(DirtyElements[SmallestDirtyParticleIndex].Payload)->DirtyGridOverflowIdx == INDEX_NONE);

			if ((!DoneWithGridElements || !DoneWithNonGridElements))
			{
				const int32 Index = SmallestDirtyParticleIndex;
				const auto& Elem = DirtyElements[Index];

				if (!Function(Elem))
				{
					return false;
				}
			}
		}
		return true;
	}

	/** Cached version of the overlap function 
	 * @param QueryBounds Bounds we want to query the tree on
	 * @param Visitor Object owning the payload and used in the overlap function to store the result
	 * @param bOverlapResult Result of the overlap function that will be used in the overlap fast
	 * @return Boolean to check if we have used or not the cached overlapping leaves (for static we don't have the cache yet)
	 */
	template <EAABBQueryType Query, typename SQVisitor>
	bool OverlapCached(const FAABB3& QueryBounds, SQVisitor& Visitor, bool& bOverlapResult) const
	{
		bool bCouldUseCache = false;
		bOverlapResult = true;
		// Only the overlap queries could use the caching
		if (Query == EAABBQueryType::Overlap && Visitor.GetQueryPayload())
		{
			// Grab the payload from the visitor (only available on physics thread for now) and retrieve the info
			const TPayloadType& QueryPayload = *static_cast<const TPayloadType*>(Visitor.GetQueryPayload());
			if( const FAABBTreePayloadInfo* QueryInfo =  PayloadToInfo.Find(QueryPayload))
			{
				const int32 LeafIndex = QueryInfo->LeafIdx;
				if(LeafIndex != INDEX_NONE && LeafIndex < (OverlappingOffsets.Num()-1))
				{
					//Once we have the leaf index we can loop over the overlapping leaves
					for(int32 OverlappingIndex = OverlappingOffsets[LeafIndex];
								OverlappingIndex < OverlappingOffsets[LeafIndex+1]; ++OverlappingIndex)
					{
						const TLeafType& OverlappingLeaf = Leaves[OverlappingLeaves[OverlappingIndex]];
						if (OverlappingLeaf.OverlapFast(QueryBounds, Visitor) == false)
						{
							bOverlapResult = false;
							break;
						}
					}
					bCouldUseCache = true;
				}
			}
		}
		return bCouldUseCache;
	}

	template <EAABBQueryType Query, typename TQueryFastData, typename SQVisitor>
	bool QueryImp(const FVec3& RESTRICT Start, TQueryFastData& CurData, const FVec3& QueryHalfExtents, const FAABB3& QueryBounds, SQVisitor& Visitor, const FVec3& Dir, const FVec3& InvDir, const bool bParallel[3]) const
	{
		PHYSICS_CSV_CUSTOM_VERY_EXPENSIVE(PhysicsCounters, MaxDirtyElements, DirtyElements.Num(), ECsvCustomStatOp::Max);
		PHYSICS_CSV_CUSTOM_VERY_EXPENSIVE(PhysicsCounters, MaxNumLeaves, Leaves.Num(), ECsvCustomStatOp::Max);
		PHYSICS_CSV_SCOPED_VERY_EXPENSIVE(PhysicsVerbose, QueryImp);
		//QUICK_SCOPE_CYCLE_COUNTER(AABBTreeQueryImp);
#if !WITH_EDITOR
		//CSV_SCOPED_TIMING_STAT(ChaosPhysicsTimers, AABBTreeQuery)
#endif
		FReal TOI = 0;
		{
			//QUICK_SCOPE_CYCLE_COUNTER(QueryGlobal);
			PHYSICS_CSV_SCOPED_VERY_EXPENSIVE(PhysicsVerbose, QueryImp_Global);
			for(const auto& Elem : GlobalPayloads)
			{
				if (PrePreFilterHelper(Elem.Payload, Visitor))
				{
					continue;
				}

				const FAABB3 InstanceBounds(Elem.Bounds.Min(), Elem.Bounds.Max());
				if(TAABBTreeIntersectionHelper<TQueryFastData,Query>::Intersects(Start, CurData, TOI, InstanceBounds, QueryBounds, QueryHalfExtents, Dir, InvDir, bParallel))
				{
					TSpatialVisitorData<TPayloadType> VisitData(Elem.Payload,true, InstanceBounds);
					bool bContinue;
					if(Query == EAABBQueryType::Overlap)
					{
						bContinue = Visitor.VisitOverlap(VisitData);
					} else
					{
						bContinue = Query == EAABBQueryType::Sweep ? Visitor.VisitSweep(VisitData,CurData) : Visitor.VisitRaycast(VisitData,CurData);
					}

					if(!bContinue)
					{
						return false;
					}
				}
			}
		}

		if (DirtyElementTree)
		{
			if (!DirtyElementTree->template QueryImp<Query, TQueryFastData, SQVisitor>(Start, CurData, QueryHalfExtents, QueryBounds, Visitor, Dir, InvDir, bParallel))
			{
				return false;
			}
		}
		else if (bMutable)
		{	// Returns true if we should continue
			auto IntersectAndVisit = [&](const FElement& Elem) -> bool
			{
				const FAABB3 InstanceBounds(Elem.Bounds.Min(), Elem.Bounds.Max());
				if (PrePreFilterHelper(Elem.Payload, Visitor))
				{
					return true;
				}

				if (TAABBTreeIntersectionHelper<TQueryFastData, Query>::Intersects(Start, CurData, TOI, InstanceBounds, QueryBounds, QueryHalfExtents, Dir, InvDir, bParallel))
				{
					TSpatialVisitorData<TPayloadType> VisitData(Elem.Payload, true, InstanceBounds);
					bool bContinue;
					if constexpr (Query == EAABBQueryType::Overlap)
					{
						bContinue = Visitor.VisitOverlap(VisitData);
					}
					else
					{
						bContinue = Query == EAABBQueryType::Sweep ? Visitor.VisitSweep(VisitData, CurData) : Visitor.VisitRaycast(VisitData, CurData);
					}

					if (!bContinue)
					{
						return false;
					}
				}
				return true;
			};

			//QUICK_SCOPE_CYCLE_COUNTER(QueryDirty);
			if (DirtyElements.Num() > 0)
			{
				bool bUseGrid = false;

				if (DirtyElementGridEnabled() && CellHashToFlatArray.Num() > 0)
				{
					if constexpr (Query == EAABBQueryType::Overlap)
					{
						bUseGrid = !TooManyOverlapQueryCells(QueryBounds, DirtyElementGridCellSizeInv, DirtyElementMaxGridCellQueryCount);
					}
					else if constexpr (Query == EAABBQueryType::Raycast)
					{
						bUseGrid = !TooManyRaycastQueryCells(Start, CurData.Dir, CurData.CurrentLength, DirtyElementGridCellSizeInv, DirtyElementMaxGridCellQueryCount);
					}
					else if constexpr (Query == EAABBQueryType::Sweep)
					{
						bUseGrid = !TooManySweepQueryCells(QueryHalfExtents, Start, CurData.Dir, CurData.CurrentLength, DirtyElementGridCellSizeInv, DirtyElementMaxGridCellQueryCount);
					}
				}

				if (bUseGrid)
				{
					PHYSICS_CSV_SCOPED_VERY_EXPENSIVE(PhysicsVerbose, QueryImp_DirtyElementsGrid);
					TArray<DirtyGridHashEntry> HashEntryForOverlappedCells;

					auto AddHashEntry = [&](int32 QueryCellHash)
					{
						const DirtyGridHashEntry* HashEntry = CellHashToFlatArray.Find(QueryCellHash);
						if (HashEntry)
						{
							HashEntryForOverlappedCells.Add(*HashEntry);
						}
						return true;
					};

					if constexpr (Query == EAABBQueryType::Overlap)
					{
						DoForOverlappedCells(QueryBounds, DirtyElementGridCellSize, DirtyElementGridCellSizeInv, AddHashEntry);
					}
					else if constexpr (Query == EAABBQueryType::Raycast)
					{
						DoForRaycastIntersectCells(Start, CurData.Dir, CurData.CurrentLength, DirtyElementGridCellSize, DirtyElementGridCellSizeInv, AddHashEntry);
					}
					else if constexpr (Query == EAABBQueryType::Sweep)
					{
						DoForSweepIntersectCells(QueryHalfExtents, Start, CurData.Dir, CurData.CurrentLength, DirtyElementGridCellSize, DirtyElementGridCellSizeInv,
							[&](FReal X, FReal Y)
							{
								int32 QueryCellHash = HashCoordinates(X, Y, DirtyElementGridCellSizeInv);
								const DirtyGridHashEntry* HashEntry = CellHashToFlatArray.Find(QueryCellHash);
								if (HashEntry)
								{
									HashEntryForOverlappedCells.Add(*HashEntry);
								}
							});
					}

					if (!DoForHitGridCellsAndOverflow(HashEntryForOverlappedCells, IntersectAndVisit))
					{
						return false;
					}
				}  // end overlap

				else
				{
					PHYSICS_CSV_SCOPED_VERY_EXPENSIVE(PhysicsVerbose, QueryImp_DirtyElements);
					for (const auto& Elem : DirtyElements)
					{
						if (!IntersectAndVisit(Elem))
						{
							return false;
						}
					}
				}
			}
		}

		struct FNodeQueueEntry
		{
			int32 NodeIdx;
			FReal TOI;
		};

		// Caching is for now only available for dynanmic tree
		if (bDynamicTree && !OverlappingLeaves.IsEmpty())
		{
			// For overlap query and dynamic tree we are using the cached overlapping leaves
			bool bOverlapResult = true;
			if (OverlapCached<Query, SQVisitor>(QueryBounds, Visitor, bOverlapResult))
			{
				return bOverlapResult;
			}
		}
		
		constexpr int32 MaxNodeStackNumOnSystemStack = 255;
		TArray<FNodeQueueEntry, TSizedInlineAllocator<MaxNodeStackNumOnSystemStack,32> > NodeStack;
		
		if (bDynamicTree)
		{

			if (RootNode != INDEX_NONE)
			{
				NodeStack.Emplace(FNodeQueueEntry{RootNode, 0});
			}
		}
		else if (Nodes.Num())
		{
			NodeStack.Emplace(FNodeQueueEntry{ 0, 0 });
		}

// Slow debug code
//#if !WITH_EDITOR
//		if (Query == EAABBQueryType::Overlap)
//		{
//			CSV_CUSTOM_STAT(ChaosPhysicsTimers, OverlapCount, 1, ECsvCustomStatOp::Accumulate);
//			CSV_CUSTOM_STAT(ChaosPhysicsTimers, DirtyCount, DirtyElements.Num(), ECsvCustomStatOp::Max);
//		}
//#endif


		VectorRegister4Double StartSimd;
		VectorRegister4Double DirSimd;
		VectorRegister4Double Parallel;
		VectorRegister4Double InvDirSimd;
		VectorRegister4Double LengthSimd;

		if constexpr (Query == EAABBQueryType::Raycast)
		{
			StartSimd = VectorLoadDouble3(&Start.X);
			DirSimd = VectorLoadDouble3(&Dir.X);
			Parallel = VectorCompareGT(GlobalVectorConstants::DoubleSmallNumber, VectorAbs(DirSimd));
			InvDirSimd = VectorBitwiseNotAnd(Parallel, VectorDivide(VectorOne(), DirSimd));
			LengthSimd = VectorSetDouble1(CurData.CurrentLength);
		}

		while (NodeStack.Num())
		{
			PHYSICS_CSV_SCOPED_VERY_EXPENSIVE(PhysicsVerbose, QueryImp_NodeTraverse);

//#if !WITH_EDITOR
//			if (Query == EAABBQueryType::Overlap)
//			{
//				CSV_CUSTOM_STAT(ChaosPhysicsTimers, AABBCheckCount, 1, ECsvCustomStatOp::Accumulate);
//			}
//#endif
			const FNodeQueueEntry NodeEntry = NodeStack.Pop(EAllowShrinking::No);
			if constexpr (Query != EAABBQueryType::Overlap)
			{
				if (NodeEntry.TOI > CurData.CurrentLength)
				{
					continue;
				}
			}

			const FNode& Node = Nodes[NodeEntry.NodeIdx];
			if (Node.bLeaf)
			{
				PHYSICS_CSV_SCOPED_VERY_EXPENSIVE(PhysicsVerbose, NodeTraverse_Leaf);
				const auto& Leaf = Leaves[Node.ChildrenNodes[0]];
				if constexpr (Query == EAABBQueryType::Overlap)
				{
					if (Leaf.OverlapFast(QueryBounds, Visitor) == false)
					{
						return false;
					}
				}
				else if constexpr (Query == EAABBQueryType::Sweep)
				{
					if (Leaf.SweepFast(Start, CurData, QueryHalfExtents, Visitor, Dir, InvDir, bParallel) == false)
					{
						return false;
					}
				}
				else if (Leaf.RaycastFastSimd(StartSimd, CurData, Visitor, DirSimd, InvDirSimd, Parallel, LengthSimd) == false)
				{
					return false;
				}
			}
			else
			{
				PHYSICS_CSV_SCOPED_VERY_EXPENSIVE(PhysicsVerbose, NodeTraverse_Branch);
				int32 Idx = 0;
				
				if constexpr (Query != EAABBQueryType::Overlap)
				{
					bool bIntersect0, bIntersect1;
					FReal TOI0, TOI1;
					if constexpr (Query == EAABBQueryType::Raycast)
					{
						FAABBVectorizedDouble AABBVectorized(Node.ChildrenBounds[0]);
						VectorRegister4Double TOISimd;
						bIntersect0 = AABBVectorized.RaycastFast(StartSimd, InvDirSimd, Parallel, LengthSimd, TOISimd);
						VectorStoreDouble1(TOISimd, &TOI0);

						AABBVectorized = FAABBVectorizedDouble(Node.ChildrenBounds[1]);
						bIntersect1 = AABBVectorized.RaycastFast(StartSimd, InvDirSimd, Parallel, LengthSimd, TOISimd);
						VectorStoreDouble1(TOISimd, &TOI1);
					}
					else
					{				
						FAABB3 AABB0(Node.ChildrenBounds[0].Min(), Node.ChildrenBounds[0].Max());
						bIntersect0 = TAABBTreeIntersectionHelper<TQueryFastData, Query>::Intersects(Start, CurData, TOI0, AABB0, QueryBounds, QueryHalfExtents, Dir, InvDir, bParallel);

						FAABB3 AABB1(Node.ChildrenBounds[1].Min(), Node.ChildrenBounds[1].Max());
						bIntersect1 = TAABBTreeIntersectionHelper<TQueryFastData, Query>::Intersects(Start, CurData, TOI1, AABB1, QueryBounds, QueryHalfExtents, Dir, InvDir, bParallel);
					}
					if (bIntersect0 && bIntersect1)
					{
						if (TOI1 > TOI0)
						{
							NodeStack.Emplace(FNodeQueueEntry{Node.ChildrenNodes[1], TOI1});
							NodeStack.Emplace(FNodeQueueEntry{Node.ChildrenNodes[0], TOI0});
						}
						else
						{
							NodeStack.Emplace(FNodeQueueEntry{Node.ChildrenNodes[0], TOI0});
							NodeStack.Emplace(FNodeQueueEntry{ Node.ChildrenNodes[1], TOI1 });
						}
					}
					else if (bIntersect0)
					{
						NodeStack.Emplace(FNodeQueueEntry{Node.ChildrenNodes[0], TOI0});
					}
					else if (bIntersect1)
					{
						NodeStack.Emplace(FNodeQueueEntry{ Node.ChildrenNodes[1], TOI1 });
					}
				}
				else
				{
					for (const TAABB<T, 3>&AABB : Node.ChildrenBounds)
					{
						if (TAABBTreeIntersectionHelper<TQueryFastData, Query>::Intersects(Start, CurData, TOI, FAABB3(AABB.Min(), AABB.Max()), QueryBounds, QueryHalfExtents, Dir, InvDir, bParallel))
						{
							NodeStack.Emplace(FNodeQueueEntry{ Node.ChildrenNodes[Idx], TOI });
						}
						++Idx;
					}
				}
			}
		}

		return true;
	}

	int32 GetNewWorkSnapshot()
	{
		if(WorkPoolFreeList.Num())
		{
			return WorkPoolFreeList.Pop();
		}
		else
		{
			return WorkPool.AddDefaulted(1);
		}
	}

	void FreeWorkSnapshot(int32 WorkSnapshotIdx)
	{
		//Reset for next time they want to use it
		WorkPool[WorkSnapshotIdx] = FWorkSnapshot();
		WorkPoolFreeList.Add(WorkSnapshotIdx);
		
	}

	template <typename TParticles>
	void GenerateTree(const TParticles& Particles)
	{
		SCOPE_CYCLE_COUNTER(STAT_AABBTreeGenerateTree);
		this->SetAsyncTimeSlicingComplete(false);

		ensure(WorkStack.Num() == 0);

		int32 NumParticles = 0;
		constexpr bool bIsValidParticleArray = !std::is_same_v<TParticles, std::nullptr_t>;
		if constexpr (bIsValidParticleArray)
		{
			NumParticles = Particles.Num();
		}

		const int32 ExpectedNumLeaves = NumParticles / MaxChildrenInLeaf;
		const int32 ExpectedNumNodes = ExpectedNumLeaves;

		WorkStack.Reserve(ExpectedNumNodes);

		const int32 CurIdx = GetNewWorkSnapshot();
		FWorkSnapshot& WorkSnapshot = WorkPool[CurIdx];
		WorkSnapshot.Elems.Reserve(NumParticles);
		
		
		GlobalPayloads.Reset();
		Leaves.Reset();
		Nodes.Reset();
		RootNode = INDEX_NONE;
		FirstFreeInternalNode = INDEX_NONE;
		FirstFreeLeafNode = INDEX_NONE;
		DirtyElements.Reset();
		CellHashToFlatArray.Reset(); 
		FlattenedCellArrayOfDirtyIndices.Reset();
		DirtyElementsGridOverflow.Reset();
		if (DirtyElementTree)
		{
			DirtyElementTree->Reset();
		}
		
		TreeStats.Reset();
		TreeExpensiveStats.Reset();
		PayloadToInfo.Reset();
		NumProcessedThisSlice = 0;

		StartSliceTimeStamp = FPlatformTime::Seconds();

		GetCVars();  // Safe to copy CVARS here

		if (bDynamicTree)
		{
			int32 Idx = 0;
			if constexpr (bIsValidParticleArray)
			{
				for (auto& Particle : Particles)
				{
					bool bHasBoundingBox = HasBoundingBox(Particle);
					auto Payload = Particle.template GetPayload<TPayloadType>(Idx);
					TAABB<T, 3> ElemBounds = ComputeWorldSpaceBoundingBox(Particle, false, (T)0);

					UpdateElement(Payload, ElemBounds, bHasBoundingBox);
					++Idx;
				}
			}
			this->SetAsyncTimeSlicingComplete(true);
			return;
		}

		WorkSnapshot.Bounds = TAABB<T, 3>::EmptyAABB();

		{
			SCOPE_CYCLE_COUNTER(STAT_AABBTreeTimeSliceSetup);

			// Prepare to find the average and scaled variance of particle centers.
			WorkSnapshot.AverageCenter = FVec3(0);
			WorkSnapshot.ScaledCenterVariance = FVec3(0);

			int32 Idx = 0;

			//TODO: we need a better way to time-slice this case since there can be a huge number of Particles. Can't do it right now without making full copy
			TVec3<T> CenterSum(0);

			if constexpr (bIsValidParticleArray)
			{
				for (auto& Particle : Particles)
				{
					bool bHasBoundingBox = HasBoundingBox(Particle);
					auto Payload = Particle.template GetPayload<TPayloadType>(Idx);
					TAABB<T, 3> ElemBounds = ComputeWorldSpaceBoundingBox(Particle, false, (T)0);

					// If bounds are bad, use global so we won't screw up splitting computations.
					if (bHasBoundingBox && ValidateBounds(ElemBounds) == false)
					{
						bHasBoundingBox = false;
						ensureMsgf(false, TEXT("AABBTree encountered invalid bounds input. Forcing element to global payload. Min: %s Max: %s."),
							*ElemBounds.Min().ToString(), *ElemBounds.Max().ToString());
					}


					if (bHasBoundingBox)
					{
						if (ElemBounds.Extents().Max() > MaxPayloadBounds)
						{
							bHasBoundingBox = false;
						}
						else
						{
							FReal NumElems = (FReal)(WorkSnapshot.Elems.Add(FElement{ Payload, ElemBounds }) + 1);
							WorkSnapshot.Bounds.GrowToInclude(ElemBounds);

							// Include the current particle in the average and scaled variance of the particle centers using Welford's method.
							TVec3<T> CenterDelta = ElemBounds.Center() - WorkSnapshot.AverageCenter;
							WorkSnapshot.AverageCenter += CenterDelta / (T)NumElems;
							WorkSnapshot.ScaledCenterVariance += (ElemBounds.Center() - WorkSnapshot.AverageCenter) * CenterDelta;
						}
					}
					else
					{
						ElemBounds = TAABB<T, 3>(TVec3<T>(TNumericLimits<T>::Lowest()), TVec3<T>(TNumericLimits<T>::Max()));
					}

					if (!bHasBoundingBox)
					{
						if (bMutable)
						{
							PayloadToInfo.Add(Payload, FAABBTreePayloadInfo{ GlobalPayloads.Num(), INDEX_NONE, INDEX_NONE, INDEX_NONE });
						}
						GlobalPayloads.Add(FElement{ Payload, ElemBounds });
					}

					++Idx;
					//todo: payload info
				}
			}
		}

		NumProcessedThisSlice = NumParticles;	//todo: give chance to time slice out of next phase

		{
			SCOPE_CYCLE_COUNTER(STAT_AABBTreeInitialTimeSlice);
			WorkSnapshot.NewNodeIdx = 0;
			WorkSnapshot.NodeLevel = 0;

			//push root onto stack
			WorkStack.Add(CurIdx);

			SplitNode();
		}

		/*  Helper validation code 
		int32 Count = 0;
		TSet<int32> Seen;
		if(WorkStack.Num() == 0)
		{
			int32 LeafIdx = 0;
			for (const auto& Leaf : Leaves)
			{
				Validate(Seen, Count, Leaf);
				bool bHasParent = false;
				for (const auto& Node : Nodes)
				{
					if (Node.bLeaf && Node.ChildrenNodes[0] == LeafIdx)
					{
						bHasParent = true;

						break;
					}
				}
				ensure(bHasParent);
				++LeafIdx;
			}
			ensure(Count == 0 || Seen.Num() == Count);
			ensure(Count == 0 || Count == Particles.Num());
		}
		*/

	}

	enum eTimeSlicePhase
	{
		PreFindBestBounds,
		DuringFindBestBounds,
		ProcessingChildren
	};

	struct FSplitInfo
	{
		TAABB<T, 3> RealBounds;	//Actual bounds as children are added
		int32 WorkSnapshotIdx;	//Idx into work snapshot pool
	};

	struct FWorkSnapshot
	{
		FWorkSnapshot()
			: TimeslicePhase(eTimeSlicePhase::PreFindBestBounds)
		{

		}

		eTimeSlicePhase TimeslicePhase;

		TAABB<T, 3> Bounds;
		TArray<FElement> Elems;

		// The average of the element centers and their variance times the number of elements.
		TVec3<T> AverageCenter;
		TVec3<T> ScaledCenterVariance;

		int32 NodeLevel;
		int32 NewNodeIdx;

		int32 BestBoundsCurIdx;

		FSplitInfo SplitInfos[2];
	};

	int32 GetLastIndexToProcess(int32 CurrentIndex)
	{
		int32 LastNodeToProcessIndex = 0;
		if (FAABBTimeSliceCVars::bUseTimeSliceMillisecondBudget)
		{
			// When TimeSlicing by a millisecond budget, try to process all nodes.
			// We will stop if we ran out of time and continue on the next frame
			LastNodeToProcessIndex = WorkPool[CurrentIndex].Elems.Num();
		}
		else
		{
			const bool WeAreTimeslicing = (MaxNumToProcess > 0);
			const int32 NumWeCanProcess = MaxNumToProcess - NumProcessedThisSlice;
			LastNodeToProcessIndex = WeAreTimeslicing ? FMath::Min(WorkPool[CurrentIndex].BestBoundsCurIdx + NumWeCanProcess, WorkPool[CurrentIndex].Elems.Num()) : WorkPool[CurrentIndex].Elems.Num();
		}

		return LastNodeToProcessIndex;
	}

	bool CanContinueProcessingNodes(bool bOnlyUseTimeStampCheck = true)
	{
		bool bCanDoWork = true;

		if (FAABBTimeSliceCVars::bUseTimeSliceMillisecondBudget)
		{
			bool bCheckIfHasAvailableTime = false;

			if (bOnlyUseTimeStampCheck)
			{
				bCheckIfHasAvailableTime = true;
			}
			else
			{
				// Checking for platform time every time is expensive, so only do it between a configured amount of elements.
				// This means we might overshoot the budget, but on the other hand we will keep most of the time doing actual work.
				CurrentProcessedNodesSinceChecked++;
				if (CurrentProcessedNodesSinceChecked > FAABBTimeSliceCVars::MinNodesChunkToProcessBetweenTimeChecks)
				{
					CurrentProcessedNodesSinceChecked = 0;
					bCheckIfHasAvailableTime = true;
				}	
			}
	
			if (bCheckIfHasAvailableTime)
			{
				const double ElapsedTime = FPlatformTime::Seconds() - StartSliceTimeStamp;
				const bool WeAreTimeslicing = FAABBTimeSliceCVars::MaxProcessingTimePerSliceSeconds > 0 && MaxNumToProcess > 0;	
				if (WeAreTimeslicing && !FMath::IsNearlyZero(StartSliceTimeStamp) && ElapsedTime > FAABBTimeSliceCVars::MaxProcessingTimePerSliceSeconds)
				{
					// done enough
					bCanDoWork = false; 
				}
			}
		}
		else
		{
			const bool WeAreTimeslicing = (MaxNumToProcess > 0);
			if (WeAreTimeslicing && (NumProcessedThisSlice >= MaxNumToProcess))
			{
				// done enough
				bCanDoWork = false;  
			}
		}

		return bCanDoWork;
	}

	void FindBestBounds(const int32 StartElemIdx, int32& InOutLastElem, FWorkSnapshot& CurrentSnapshot, int32 MaxAxis, const TVec3<T>& SplitCenter)
	{
		const T SplitVal = SplitCenter[MaxAxis];

		// add all elements to one of the two split infos at this level - root level [ not taking into account the max number allowed or anything
		for (int32 ElemIdx = StartElemIdx; ElemIdx < InOutLastElem; ++ElemIdx)
		{
			const FElement& Elem = CurrentSnapshot.Elems[ElemIdx];
			int32 BoxIdx = 0;
			const TVec3<T> ElemCenter = Elem.Bounds.Center();
			
			// This was changed to work around a code generation issue on some platforms, don't change it without testing results of ElemCenter computation.
			// 
			// NOTE: This needs review.  TVec3<T>::operator[] should now cope with the strict aliasing violation which caused the original codegen breakage.
			T CenterVal = ElemCenter[0];
			if (MaxAxis == 1)
			{
				CenterVal = ElemCenter[1];
			}
			else if(MaxAxis == 2)
			{
				CenterVal = ElemCenter[2];
			}
		
			const int32 MinBoxIdx = CenterVal <= SplitVal ? 0 : 1;
			
			FSplitInfo& SplitInfo = CurrentSnapshot.SplitInfos[MinBoxIdx];
			FWorkSnapshot& WorkSnapshot = WorkPool[SplitInfo.WorkSnapshotIdx];
			T NumElems = (T)(WorkSnapshot.Elems.Add(Elem) + 1);
			SplitInfo.RealBounds.GrowToInclude(Elem.Bounds);

			// Include the current particle in the average and scaled variance of the particle centers using Welford's method.
			TVec3<T> CenterDelta = ElemCenter - WorkSnapshot.AverageCenter;
			WorkSnapshot.AverageCenter += CenterDelta / NumElems;
			WorkSnapshot.ScaledCenterVariance += (ElemCenter - WorkSnapshot.AverageCenter) * CenterDelta;

			constexpr bool bOnlyUSeTimeStampCheck = false;
			if (!CanContinueProcessingNodes(bOnlyUSeTimeStampCheck))
			{
				// If we ended before processing all the requested nodes, update the out last element index variable
				const bool bIsProcessingLastRequestedIndex = ElemIdx == InOutLastElem - 1;
				InOutLastElem = bIsProcessingLastRequestedIndex ? InOutLastElem : ElemIdx + 1;
				break; // done enough
			}
		}

		NumProcessedThisSlice += InOutLastElem - StartElemIdx;
	}
	
	void SplitNode()
	{		
		while (WorkStack.Num())
		{
			//NOTE: remember to be careful with this since it's a pointer on a tarray
			const int32 CurIdx = WorkStack.Last();

			if (WorkPool[CurIdx].TimeslicePhase == eTimeSlicePhase::ProcessingChildren)
			{
				//If we got to this it must be that my children are done, so I'm done as well
				WorkStack.Pop(EAllowShrinking::No);
				FreeWorkSnapshot(CurIdx);
				continue;
			}

			const int32 NewNodeIdx = WorkPool[CurIdx].NewNodeIdx;

			// create the actual node space but might no be filled in (YET) due to time slicing exit
			if (NewNodeIdx >= Nodes.Num())
			{
				Nodes.AddDefaulted((1 + NewNodeIdx) - Nodes.Num());
			}

			if (!CanContinueProcessingNodes())
			{
				return; // done enough
			}

			auto& PayloadToInfoRef = PayloadToInfo;
			auto& LeavesRef = Leaves;
			auto& NodesRef = Nodes;
			auto& WorkPoolRef = WorkPool;
			auto& TreeExpensiveStatsRef = TreeExpensiveStats;
			auto MakeLeaf = [NewNodeIdx, &PayloadToInfoRef, &WorkPoolRef, CurIdx, &LeavesRef, &NodesRef, &TreeExpensiveStatsRef]()
			{
				if (bMutable)
				{
					//todo: does this need time slicing in the case when we have a ton of elements that can't be split?
					//hopefully not a real issue
					for (const FElement& Elem : WorkPoolRef[CurIdx].Elems)
					{
						PayloadToInfoRef.Add(Elem.Payload, FAABBTreePayloadInfo{ INDEX_NONE, INDEX_NONE, LeavesRef.Num() });
					}
				}

				NodesRef[NewNodeIdx].bLeaf = true;
				NodesRef[NewNodeIdx].ChildrenNodes[0] = LeavesRef.Add(TLeafType{ WorkPoolRef[CurIdx].Elems }); //todo: avoid copy?
			};

			if (WorkPool[CurIdx].Elems.Num() <= MaxChildrenInLeaf || WorkPool[CurIdx].NodeLevel >= MaxTreeDepth)
			{

				MakeLeaf();
				WorkStack.Pop(EAllowShrinking::No);	//finished with this node
				FreeWorkSnapshot(CurIdx);
				continue;
			}

			if (WorkPool[CurIdx].TimeslicePhase == eTimeSlicePhase::PreFindBestBounds)
			{
				//Add two children, remember this invalidates any pointers to current snapshot
				const int32 FirstChildIdx = GetNewWorkSnapshot();
				const int32 SecondChildIdx = GetNewWorkSnapshot();

				//mark idx of children into the work pool
				WorkPool[CurIdx].SplitInfos[0].WorkSnapshotIdx = FirstChildIdx;
				WorkPool[CurIdx].SplitInfos[1].WorkSnapshotIdx = SecondChildIdx;

				for (FSplitInfo& SplitInfo : WorkPool[CurIdx].SplitInfos)
				{
					SplitInfo.RealBounds = TAABB<T, 3>::EmptyAABB();
				}

				WorkPool[CurIdx].BestBoundsCurIdx = 0;
				WorkPool[CurIdx].TimeslicePhase = eTimeSlicePhase::DuringFindBestBounds;
				const int32 ExpectedNumPerChild = WorkPool[CurIdx].Elems.Num() / 2;
				{
					WorkPool[FirstChildIdx].Elems.Reserve(ExpectedNumPerChild);
					WorkPool[SecondChildIdx].Elems.Reserve(ExpectedNumPerChild);

					// Initialize the the two info's element center average and scaled variance.
					WorkPool[FirstChildIdx].AverageCenter = TVec3<T>(0);
					WorkPool[FirstChildIdx].ScaledCenterVariance = TVec3<T>(0);
					WorkPool[SecondChildIdx].AverageCenter = TVec3<T>(0);
					WorkPool[SecondChildIdx].ScaledCenterVariance = TVec3<T>(0);
				}
				
				if (!CanContinueProcessingNodes())
				{
					// done enough
					return; 
				}
			}

			if (WorkPool[CurIdx].TimeslicePhase == eTimeSlicePhase::DuringFindBestBounds)
			{
				int32 LastIdxToProcess = GetLastIndexToProcess(CurIdx);

				// Determine the axis to split the AABB on based on the SplitOnVarianceAxis console variable. If it is not 1, simply use the largest axis
				// of the work snapshot bounds; otherwise, select the axis with the greatest center variance. Note that the variance times the number of
				// elements is actually used but since all that is needed is the axis with the greatest variance the scale factor is irrelevant.
				const int32 MaxAxis = (FAABBTreeCVars::SplitOnVarianceAxis != 1) ? WorkPool[CurIdx].Bounds.LargestAxis() :
					(WorkPool[CurIdx].ScaledCenterVariance[0] > WorkPool[CurIdx].ScaledCenterVariance[1] ?
						(WorkPool[CurIdx].ScaledCenterVariance[0] > WorkPool[CurIdx].ScaledCenterVariance[2] ? 0 : 2) :
						(WorkPool[CurIdx].ScaledCenterVariance[1] > WorkPool[CurIdx].ScaledCenterVariance[2] ? 1 : 2));

				// Find the point where the AABB will be split based on the SplitAtAverageCenter console variable. If it is not 1, just use the center
				// of the AABB; otherwise, use the average of the element centers.
				const TVec3<T>& Center = (FAABBTreeCVars::SplitAtAverageCenter != 1) ? WorkPool[CurIdx].Bounds.Center() : WorkPool[CurIdx].AverageCenter;

				FindBestBounds(WorkPool[CurIdx].BestBoundsCurIdx, LastIdxToProcess, WorkPool[CurIdx], MaxAxis, Center);
				WorkPool[CurIdx].BestBoundsCurIdx = LastIdxToProcess;
				
				if (!CanContinueProcessingNodes())
				{
					// done enough
					return; 
				}
			}

			const int32 FirstChildIdx = WorkPool[CurIdx].SplitInfos[0].WorkSnapshotIdx;
			const int32 SecondChildIdx = WorkPool[CurIdx].SplitInfos[1].WorkSnapshotIdx;

			const bool bChildrenInBothHalves = WorkPool[FirstChildIdx].Elems.Num() && WorkPool[SecondChildIdx].Elems.Num();

			// if children in both halves, push them on the stack to continue the split
			if (bChildrenInBothHalves)
			{
				Nodes[NewNodeIdx].bLeaf = false;

				Nodes[NewNodeIdx].ChildrenBounds[0] = WorkPool[CurIdx].SplitInfos[0].RealBounds;
				WorkPool[FirstChildIdx].Bounds = Nodes[NewNodeIdx].ChildrenBounds[0];
				Nodes[NewNodeIdx].ChildrenNodes[0] = Nodes.Num();

				Nodes[NewNodeIdx].ChildrenBounds[1] = WorkPool[CurIdx].SplitInfos[1].RealBounds;
				WorkPool[SecondChildIdx].Bounds = Nodes[NewNodeIdx].ChildrenBounds[1];
				Nodes[NewNodeIdx].ChildrenNodes[1] = Nodes.Num() + 1;

				WorkPool[FirstChildIdx].NodeLevel = WorkPool[CurIdx].NodeLevel + 1;
				WorkPool[SecondChildIdx].NodeLevel = WorkPool[CurIdx].NodeLevel + 1;

				WorkPool[FirstChildIdx].NewNodeIdx = Nodes[NewNodeIdx].ChildrenNodes[0];
				WorkPool[SecondChildIdx].NewNodeIdx = Nodes[NewNodeIdx].ChildrenNodes[1];

				//push these two new nodes onto the stack
				WorkStack.Add(SecondChildIdx);
				WorkStack.Add(FirstChildIdx);

				// create the actual node so that no one else can use our children node indices
				const int32 HighestNodeIdx = Nodes[NewNodeIdx].ChildrenNodes[1];
				Nodes.AddDefaulted((1 + HighestNodeIdx) - Nodes.Num());
			
				WorkPool[CurIdx].TimeslicePhase = eTimeSlicePhase::ProcessingChildren;
			}
			else
			{
				//couldn't split so just make a leaf - THIS COULD CONTAIN MORE THAN MaxChildrenInLeaf!!!
				MakeLeaf();
				WorkStack.Pop(EAllowShrinking::No);	//we are done with this node
				FreeWorkSnapshot(CurIdx);
			}
		}

		check(WorkStack.Num() == 0);
		//Stack is empty, clean up pool and mark task as complete
		
		this->SetAsyncTimeSlicingComplete(true);
	}

	TArray<TPayloadType> FindAllIntersectionsImp(const FAABB3& Intersection) const
	{
		struct FSimpleVisitor
		{
			FSimpleVisitor(TArray<TPayloadType>& InResults) : CollectedResults(InResults) {}
			bool VisitOverlap(const TSpatialVisitorData<TPayloadType>& Instance)
			{
				CollectedResults.Add(Instance.Payload);
				return true;
			}
			bool VisitSweep(const TSpatialVisitorData<TPayloadType>& Instance, FQueryFastData& CurData)
			{
				check(false);
				return true;
			}
			bool VisitRaycast(const TSpatialVisitorData<TPayloadType>& Instance, FQueryFastData& CurData)
			{
				check(false);
				return true;
			}

			bool PrePreFilter(const TPayloadType&) const { return false; }

			/** Return a pointer to the payload on which we are querying the acceleration structure */
			const void* GetQueryPayload() const 
			{ 
				return nullptr; 
			}

			bool HasBlockingHit() const
			{
				return false;
			}

			TArray<TPayloadType>& CollectedResults;
		};

		TArray<TPayloadType> Results;
		FSimpleVisitor Collector(Results);
		Overlap(Intersection, Collector);

		return Results;
	}

	// Set InOutMaxSize to less than 0 for unlimited
	// 
	template<typename ContainerType>
	static void AddToContainerHelper(const ContainerType& ContainerFrom, ContainerType& ContainerTo, int32 Index)
	{
		if constexpr (std::is_same_v<ContainerType, TArrayAsMap<TPayloadType, FAABBTreePayloadInfo>>)
		{
			ContainerTo.AddFrom(ContainerFrom, Index);
		}
		else if constexpr(std::is_same_v<ContainerType, TSQMap<TPayloadType, FAABBTreePayloadInfo>>)
		{
			ContainerTo.AddFrom(ContainerFrom, Index);
		}
		else
		{
			ContainerTo.Add(ContainerFrom[Index]);
		}
	}

	template<typename ContainerType>
	static int32 ContainerElementSizeHelper(const ContainerType& Container, int32 Index)
	{
		if constexpr (std::is_same_v<ContainerType, TArray<TAABBTreeLeafArray<TPayloadType, true>>>)
		{
			return sizeof(typename TArray<TAABBTreeLeafArray<TPayloadType, true>>::ElementType) + sizeof(typename decltype(Container[Index].Elems)::ElementType) * Container[Index].GetElementCount();
		}
		else if constexpr (std::is_same_v<ContainerType, TArray<TAABBTreeLeafArray<TPayloadType, false>>>)
		{
			return sizeof(typename TArray<TAABBTreeLeafArray<TPayloadType, false>>::ElementType) + sizeof(typename decltype(Container[Index].Elems)::ElementType) * Container[Index].GetElementCount();
		}
		else
		{
			return sizeof(typename ContainerType::ElementType);
		}
	}

	template<typename ContainerType, typename TCanContinueCallback>
	static bool ContinueTimeSliceCopy(const ContainerType& ContainerFrom, ContainerType& ContainerTo, int32& InOutMaxSize, const TCanContinueCallback& CanContinueCallback)
	{
		int32 SizeCopied = 0;

		for (int32 Index = ContainerTo.Num(); Index < ContainerFrom.Num() && CanContinueCallback(InOutMaxSize, SizeCopied); Index++)
		{
			AddToContainerHelper(ContainerFrom, ContainerTo, Index);
			SizeCopied += ContainerElementSizeHelper(ContainerFrom, Index);
		}

		// Update the maximum size left
		if (InOutMaxSize > 0)
		{
			if (SizeCopied > InOutMaxSize)
			{
				InOutMaxSize = 0;
			}
			else
			{
				InOutMaxSize -= SizeCopied;
			}
		}

		bool Done = ContainerTo.Num() == ContainerFrom.Num();
		return Done;
	}


#if !UE_BUILD_SHIPPING
	virtual void DebugDraw(ISpacialDebugDrawInterface<T>* InInterface) const override
	{
		if (InInterface)
		{
			int32 RootNodeIndex = bDynamicTree ? RootNode : 0;
			if (Nodes.Num() > 0 && Nodes.IsValidIndex(RootNodeIndex))
			{
				Nodes[RootNodeIndex].DebugDraw(*InInterface, Nodes, { 1.f, 1.f, 1.f }, 5.f);
			}
			for (int LeafIndex = 0; LeafIndex < Leaves.Num(); LeafIndex++)
			{
				const TLeafType& Leaf = Leaves[LeafIndex];
				Leaf.DebugDrawLeaf(*InInterface, FLinearColor::MakeRandomColor(), 10.f);
			}
			if (DirtyElementTree)
			{
				DirtyElementTree->DebugDraw(InInterface);
			}
		}
	}
#endif


	TAABBTree(const TAABBTree& Other)
		: ISpatialAcceleration<TPayloadType, T, 3>(StaticType)
		, Nodes(Other.Nodes)
		, Leaves(Other.Leaves)
		, DirtyElements(Other.DirtyElements)
		, bDynamicTree(Other.bDynamicTree)
		, RootNode(Other.RootNode)
		, FirstFreeInternalNode(Other.FirstFreeInternalNode)
		, FirstFreeLeafNode(Other.FirstFreeLeafNode)
		, CellHashToFlatArray(Other.CellHashToFlatArray)
		, FlattenedCellArrayOfDirtyIndices(Other.FlattenedCellArrayOfDirtyIndices)
		, DirtyElementsGridOverflow(Other.DirtyElementsGridOverflow)
		, DirtyElementTree(nullptr)
		, DirtyElementGridCellSize(Other.DirtyElementGridCellSize)
		, DirtyElementGridCellSizeInv(Other.DirtyElementGridCellSizeInv)
		, DirtyElementMaxGridCellQueryCount(Other.DirtyElementMaxGridCellQueryCount)
		, DirtyElementMaxPhysicalSizeInCells(Other.DirtyElementMaxPhysicalSizeInCells)
		, DirtyElementMaxCellCapacity(Other.DirtyElementMaxCellCapacity)
		, TreeStats(Other.TreeStats)
		, TreeExpensiveStats(Other.TreeExpensiveStats)
		, GlobalPayloads(Other.GlobalPayloads)
		, PayloadToInfo(Other.PayloadToInfo)
		, MaxChildrenInLeaf(Other.MaxChildrenInLeaf)
		, MaxTreeDepth(Other.MaxTreeDepth)
		, MaxPayloadBounds(Other.MaxPayloadBounds)
		, MaxNumToProcess(Other.MaxNumToProcess)
		, NumProcessedThisSlice(Other.NumProcessedThisSlice)
		, StartSliceTimeStamp(Other.StartSliceTimeStamp)
		, bModifyingTreeMultiThreadingFastCheck(Other.bModifyingTreeMultiThreadingFastCheck)
		, bShouldRebuild(Other.bShouldRebuild)
		, bBuildOverlapCache(Other.bBuildOverlapCache)		
		, OverlappingLeaves(Other.OverlappingLeaves)
		, OverlappingOffsets(Other.OverlappingOffsets)
		, OverlappingPairs(Other.OverlappingPairs)
		, OverlappingCounts(Other.OverlappingCounts)
	{
		ensure(bDynamicTree == Other.IsTreeDynamic());
		PriorityQ.Reserve(32);
		if (Other.DirtyElementTree)
		{
			DirtyElementTree = TUniquePtr<TAABBTree>(new TAABBTree(*(Other.DirtyElementTree)));
		}
	}

	virtual void DeepAssign(const ISpatialAcceleration<TPayloadType, T, 3>& Other) override
	{
		check(Other.GetType() == ESpatialAcceleration::AABBTree);
		*this = static_cast<const TAABBTree&>(Other);
	}

	TAABBTree& operator=(const TAABBTree& Rhs)
	{
		ISpatialAcceleration<TPayloadType, T, 3>::DeepAssign(Rhs);
		ensure(Rhs.WorkStack.Num() == 0);
		//ensure(Rhs.WorkPool.Num() == 0);
		//ensure(Rhs.WorkPoolFreeList.Num() == 0);
		WorkStack.Empty();
		WorkPool.Empty();
		WorkPoolFreeList.Empty();
		if(this != &Rhs)
		{
			Nodes = Rhs.Nodes;
			Leaves = Rhs.Leaves;
			DirtyElements = Rhs.DirtyElements;
			bDynamicTree = Rhs.bDynamicTree;
			RootNode = Rhs.RootNode;
			FirstFreeInternalNode = Rhs.FirstFreeInternalNode;
			FirstFreeLeafNode = Rhs.FirstFreeLeafNode;
			
			CellHashToFlatArray = Rhs.CellHashToFlatArray;
			FlattenedCellArrayOfDirtyIndices = Rhs.FlattenedCellArrayOfDirtyIndices;
			DirtyElementsGridOverflow = Rhs.DirtyElementsGridOverflow;
			TreeStats = Rhs.TreeStats;
			TreeExpensiveStats = Rhs.TreeExpensiveStats;
			
			DirtyElementGridCellSize = Rhs.DirtyElementGridCellSize;
			DirtyElementGridCellSizeInv = Rhs.DirtyElementGridCellSizeInv;
			DirtyElementMaxGridCellQueryCount = Rhs.DirtyElementMaxGridCellQueryCount;
			DirtyElementMaxPhysicalSizeInCells = Rhs.DirtyElementMaxPhysicalSizeInCells;
			DirtyElementMaxCellCapacity = Rhs.DirtyElementMaxCellCapacity;
			
			GlobalPayloads = Rhs.GlobalPayloads;
			PayloadToInfo = Rhs.PayloadToInfo;
			MaxChildrenInLeaf = Rhs.MaxChildrenInLeaf;
			MaxTreeDepth = Rhs.MaxTreeDepth;
			MaxPayloadBounds = Rhs.MaxPayloadBounds;
			MaxNumToProcess = Rhs.MaxNumToProcess;
			NumProcessedThisSlice = Rhs.NumProcessedThisSlice;
			StartSliceTimeStamp = Rhs.StartSliceTimeStamp;
			bModifyingTreeMultiThreadingFastCheck = Rhs.bModifyingTreeMultiThreadingFastCheck;
			bShouldRebuild = Rhs.bShouldRebuild;
			bBuildOverlapCache = Rhs.bBuildOverlapCache;			
			if (Rhs.DirtyElementTree)
			{
				if (!DirtyElementTree)
				{
					DirtyElementTree = TUniquePtr<TAABBTree>(new TAABBTree());
				}				
				*DirtyElementTree = *Rhs.DirtyElementTree;
			}
			else
			{
				DirtyElementTree = nullptr;
			}
		}

		return *this;
	}

	TArray<FNode> Nodes;
	TLeafContainer<TLeafType> Leaves;
	TArray<FElement> DirtyElements;

	// DynamicTree members
	bool bDynamicTree = false;
	int32 RootNode = INDEX_NONE;
	// Free lists (indices are in Nodes array)
	int32 FirstFreeInternalNode = INDEX_NONE;
	int32 FirstFreeLeafNode = INDEX_NONE;

	// Data needed for DirtyElement2DAccelerationGrid
	TMap<int32, DirtyGridHashEntry> CellHashToFlatArray; // Index, size into flat grid structure (FlattenedCellArrayOfDirtyIndices)
	TArray<int32> FlattenedCellArrayOfDirtyIndices; // Array of indices into dirty Elements array, indices are always sorted within a 2D cell
	TArray<int32> DirtyElementsGridOverflow; // Array of indices of DirtyElements that is not in the grid for some reason

	// Members for using a dynamic tree as a dirty element acceleration structure
	TUniquePtr<TAABBTree> DirtyElementTree;

	// Copy of CVARS
	T DirtyElementGridCellSize;
	T DirtyElementGridCellSizeInv;
	int32 DirtyElementMaxGridCellQueryCount;
	int32 DirtyElementMaxPhysicalSizeInCells;
	int32 DirtyElementMaxCellCapacity;
	// Some useful statistics
	AABBTreeStatistics TreeStats;
	AABBTreeExpensiveStatistics TreeExpensiveStats;

	TArray<FElement> GlobalPayloads;
	typename StorageTraits::PayloadToInfoType PayloadToInfo;

	int32 MaxChildrenInLeaf;
	int32 MaxTreeDepth;
	T MaxPayloadBounds;
	int32 MaxNumToProcess;
	int32 NumProcessedThisSlice;

	double StartSliceTimeStamp = 0.0;
	int32 CurrentProcessedNodesSinceChecked = 0;
	int32 CurrentDataElementsCopiedSinceLastCheck = 0;
	
	TArray<int32> WorkStack;
	TArray<int32> WorkPoolFreeList;
	TArray<FWorkSnapshot> WorkPool;

	bool bModifyingTreeMultiThreadingFastCheck; // Used for fast but not perfect multithreading sanity check

	bool bShouldRebuild;  // Contract: this can only ever be cleared by calling the ClearShouldRebuild method

	bool bBuildOverlapCache;
	/** Flat array of overlapping leaves.  */
	TArray<int32> OverlappingLeaves;

	/** Offset for each leaf in the overlapping leaves (OverlappingOffsets[LeafIndex]/OverlappingOffsets[LeafIndex+1] will be start/end indices of the leaf index in the overallping leaves array )*/
	TArray<int32> OverlappingOffsets;

	/** List of leaves intersecting pairs that will be used to build the flat arrays (offsets,leaves) */
	TArray<FIntVector2> OverlappingPairs;

	/** Number of overlapping leaves per leaf */
	TArray<int32> OverlappingCounts;

	/** 
	 * Buffers for calculating the best candidate for tree insertion, reset per-calculation to avoid over allocation 
	 * Tuple of Node/Index/Cost here avoids cache miss when accessing the node and its cost.
	 */
	using FNodeIndexAndCost = TTuple<FNode&, int32, FReal>;
	TArray<FNodeIndexAndCost> PriorityQ;	

	friend ::FChaosVDDataWrapperUtils;
};

template<typename TPayloadType, typename TLeafType, bool bMutable, typename T, typename StorageTraits>
FArchive& operator<<(FChaosArchive& Ar, TAABBTree<TPayloadType, TLeafType, bMutable, T, StorageTraits>& AABBTree)
{
	AABBTree.Serialize(Ar);
	return Ar;
}

}
