// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/AABB.h"
#include "Chaos/Defines.h"
#include "ChaosLog.h"
#include <limits>

namespace Chaos
{

	FORCEINLINE_DEBUGGABLE uint32 InterleaveWithZeros(uint16 input)
	{
		uint32 Intermediate = (uint32)input;
		Intermediate = (Intermediate ^ (Intermediate << 8)) & 0x00ff00ff;
		Intermediate = (Intermediate ^ (Intermediate << 4)) & 0x0f0f0f0f;
		Intermediate = (Intermediate ^ (Intermediate << 2)) & 0x33333333;
		Intermediate = (Intermediate ^ (Intermediate << 1)) & 0x55555555;
		return Intermediate;
	}

	FORCEINLINE_DEBUGGABLE int64 GetDirtyCellIndexFromWorldCoordinate(FReal Coordinate, FReal DirtyElementGridCellSizeInv)
	{
		FReal CellIndex = Coordinate * DirtyElementGridCellSizeInv;
		// Bias values just enough to remove floating point integer inconsistencies
		// Assume maximum of 1ULP error in DirtyElementGridCellSizeInv
		FReal FloatingPointMaxCellIndexError = FMath::Abs(CellIndex) * std::numeric_limits<FReal>::epsilon();
		return (int64)(FMath::Floor(CellIndex + FloatingPointMaxCellIndexError));
	}

	FORCEINLINE_DEBUGGABLE int32 HashCell(int64 XCell, int64 YCell)
	{
		return (int32)(InterleaveWithZeros((uint16)XCell) | (InterleaveWithZeros((uint16)YCell) << 1));
	}

	FORCEINLINE_DEBUGGABLE int32 HashCoordinates(FReal Xcoordinate, FReal Ycoordinate, FReal DirtyElementGridCellSizeInv)
	{
		// Requirement: Hash should change for adjacent cells
		int64 X = GetDirtyCellIndexFromWorldCoordinate(Xcoordinate,DirtyElementGridCellSizeInv);
		int64 Y = GetDirtyCellIndexFromWorldCoordinate(Ycoordinate,DirtyElementGridCellSizeInv);

		return HashCell(X, Y);
	}

	template <typename T>
	FORCEINLINE_DEBUGGABLE bool TooManyOverlapQueryCells(const TAABB<T, 3>& AABB, FReal DirtyElementGridCellSizeInv, int32 MaximumOverlap)
	{
		const int64 CellStartX = GetDirtyCellIndexFromWorldCoordinate(AABB.Min().X, DirtyElementGridCellSizeInv);
		const int64 CellStartY = GetDirtyCellIndexFromWorldCoordinate(AABB.Min().Y, DirtyElementGridCellSizeInv);

		const int64 CellEndX = GetDirtyCellIndexFromWorldCoordinate(AABB.Max().X, DirtyElementGridCellSizeInv);
		const int64 CellEndY = GetDirtyCellIndexFromWorldCoordinate(AABB.Max().Y, DirtyElementGridCellSizeInv);

		if(ensure(CellEndX >= CellStartX && CellEndY >= CellStartY))
		{
			const uint64 XsampleCount = (CellEndX - CellStartX) + 1;
			const uint64 YsampleCount = (CellEndY - CellStartY) + 1;
			if((XsampleCount <= (uint64)MaximumOverlap) &&
			   (YsampleCount <= (uint64)MaximumOverlap) &&
			   (XsampleCount * YsampleCount <= (uint64)MaximumOverlap))
			{
				return false;
			}
		}
		return true;
	}

	template <typename T, typename FunctionType>
	FORCEINLINE_DEBUGGABLE bool DoForOverlappedCells(const TAABB<T, 3>& AABB, FReal DirtyElementGridCellSize, FReal DirtyElementGridCellSizeInv, FunctionType Function)
	{
		int64 CellStartX = GetDirtyCellIndexFromWorldCoordinate(AABB.Min().X, DirtyElementGridCellSizeInv);
		int64 CellStartY = GetDirtyCellIndexFromWorldCoordinate(AABB.Min().Y, DirtyElementGridCellSizeInv);

		int64 CellEndX = GetDirtyCellIndexFromWorldCoordinate(AABB.Max().X, DirtyElementGridCellSizeInv);
		int64 CellEndY = GetDirtyCellIndexFromWorldCoordinate(AABB.Max().Y, DirtyElementGridCellSizeInv);

		for (int64 X = CellStartX; X <= CellEndX; X++)
		{
			for (int64 Y = CellStartY; Y <= CellEndY; Y++)
			{
				if (!Function(HashCell(X, Y)))
				{
					return false; // early out requested by the lambda
				}
			}
		}
		return true;
	}

	// Only execute function for new Cells not covered in old (Set difference: {Cells spanned by AABB} - { Cells spanned by AABBExclude})
	template <typename T, typename FunctionType>
	FORCEINLINE_DEBUGGABLE bool DoForOverlappedCellsExclude(const TAABB<T, 3>& AABB, const TAABB<T, 3>& AABBExclude, FReal DirtyElementGridCellSize, FReal DirtyElementGridCellSizeInv, FunctionType Function)
	{

		int64 NewCellStartX = GetDirtyCellIndexFromWorldCoordinate((FReal)AABB.Min().X, DirtyElementGridCellSizeInv);
		int64 NewCellStartY = GetDirtyCellIndexFromWorldCoordinate((FReal)AABB.Min().Y, DirtyElementGridCellSizeInv);

		int64 NewCellEndX = GetDirtyCellIndexFromWorldCoordinate((FReal)AABB.Max().X, DirtyElementGridCellSizeInv);
		int64 NewCellEndY = GetDirtyCellIndexFromWorldCoordinate((FReal)AABB.Max().Y, DirtyElementGridCellSizeInv);

		int64 OldCellStartX = GetDirtyCellIndexFromWorldCoordinate((FReal)AABBExclude.Min().X, DirtyElementGridCellSizeInv);
		int64 OldCellStartY = GetDirtyCellIndexFromWorldCoordinate((FReal)AABBExclude.Min().Y, DirtyElementGridCellSizeInv);

		int64 OldCellEndX = GetDirtyCellIndexFromWorldCoordinate((FReal)AABBExclude.Max().X, DirtyElementGridCellSizeInv);
		int64 OldCellEndY = GetDirtyCellIndexFromWorldCoordinate((FReal)AABBExclude.Max().Y, DirtyElementGridCellSizeInv);

		// Early out here
		if (OldCellStartX <= NewCellStartX &&
			OldCellStartY <= NewCellStartY &&
			OldCellEndX >= NewCellEndX &&
			OldCellEndY >= NewCellEndY)
		{
			return true;
		}

		for (int64 X = NewCellStartX; X <= NewCellEndX; X++)
		{
			for (int64 Y = NewCellStartY; Y <= NewCellEndY; Y++)
			{
				if (!(X >= OldCellStartX && X <= OldCellEndX && Y >= OldCellStartY && Y <= OldCellEndY))
				{
					if (!Function(HashCell(X, Y)))
					{
						return false; // early out requested by the lambda
					}
				}
			}
		}
		return true;
	}

	FORCEINLINE_DEBUGGABLE int32 FindInSortedArray(const TArray<int32>& Array, int32 FindValue, int32 StartIndex, int32 EndIndex)
	{
		int32 TestIndex = (EndIndex + StartIndex) / 2;
		int32 TestValue = Array[TestIndex];
		if (TestValue == FindValue)
		{
			return TestIndex;
		}

		if (StartIndex == EndIndex)
		{
			return INDEX_NONE;
		}

		if (TestValue < FindValue)
		{
			// tail-recursion
			return FindInSortedArray(Array, FindValue, TestIndex + 1, EndIndex);
		}

		if (StartIndex == TestIndex)
		{
			return INDEX_NONE;
		}

		// tail-recursion
		return FindInSortedArray(Array, FindValue, StartIndex, TestIndex - 1);
	}

	FORCEINLINE_DEBUGGABLE int32 FindInsertIndexIntoSortedArray(const TArray<int32>& Array, int32 FindValue, int32 StartIndex, int32 EndIndex)
	{
		int32 TestIndex = (EndIndex + StartIndex) / 2;
		int32 TestValue = Array[TestIndex];
		if (TestValue == FindValue)
		{
			return INDEX_NONE; // Already in array
		}

		if (StartIndex == EndIndex)
		{
			return FindValue > TestValue ? StartIndex + 1 : StartIndex;
		}

		if (TestValue < FindValue)
		{
			// tail-recursion
			return FindInsertIndexIntoSortedArray(Array, FindValue, TestIndex + 1, EndIndex);
		}

		if (StartIndex == TestIndex)
		{
			return TestIndex;
		}

		// tail-recursion
		return FindInsertIndexIntoSortedArray(Array, FindValue, StartIndex, TestIndex - 1);
	}

	// Prerequisites: The array must be sorted from StartIndex to StartIndex + Count -1, and must have one element past StartIndex + Count -1 allocated
	// returns false if the value was already in the array and therefore not added again
	FORCEINLINE_DEBUGGABLE bool InsertValueIntoSortedSubArray(TArray<int32>& Array, int32 Value, int32 StartIndex, int32 Count)
	{
		// We must keep everything sorted
		if (Count > 0)
		{
			int32 EndIndex = StartIndex + Count - 1;
			int32 InsertIndex = FindInsertIndexIntoSortedArray(Array, Value, StartIndex, EndIndex);
			//ensure(InsertIndex != INDEX_NONE) // Just for debugging
			if (InsertIndex != INDEX_NONE)
			{
				for (int32 Index = EndIndex + 1; Index > InsertIndex; Index--)
				{
					Array[Index] = Array[Index - 1];
				}
				Array[InsertIndex] = Value;
			}
			else
			{
				return false;
			}
		}
		else
		{
			Array[StartIndex] = Value;
		}
		return true;
	}

	// Prerequisites: The array must be sorted from StartIndex to EndIndex. 
	// The extra element won't be deallocated
	// returns true if the element has been found and successfully deleted
	FORCEINLINE_DEBUGGABLE bool DeleteValueFromSortedSubArray(TArray<int32>& Array, int32 Value, int32 StartIndex, int32 Count)
	{
		if (ensure(Count > 0))
		{
			int32 EndIndex = StartIndex + Count - 1;
			int32 DeleteIndex = FindInSortedArray(Array, Value, StartIndex, EndIndex);
			if (DeleteIndex != INDEX_NONE)
			{
				for (int32 Index = DeleteIndex; Index < EndIndex; Index++)
				{
					Array[Index] = Array[Index + 1];
				}
				return true;
			}
		}
		return false;
	}

	FORCEINLINE_DEBUGGABLE bool TooManySweepQueryCells(const TVec3<FReal>& QueryHalfExtents, const FVec3& StartPoint, const FVec3& Dir, FReal Length, FReal DirtyElementGridCellSizeInv, int32 DirtyElementMaxGridCellQueryCount)
	{
		if(Length > (FReal)TNumericLimits<uint64>::Max())
		{
			return true;
		}

		const uint64 EstimatedNumberOfCells =
			((uint64)(QueryHalfExtents.X * 2 * DirtyElementGridCellSizeInv) + 2) * ((uint64)(QueryHalfExtents.Y * 2 * DirtyElementGridCellSizeInv) + 2) +
			((uint64)(FMath::Max(QueryHalfExtents.X, QueryHalfExtents.Y) * 2 * DirtyElementGridCellSizeInv) + 2) * ((uint64)(Length * DirtyElementGridCellSizeInv) + 2);

		return EstimatedNumberOfCells > (uint64)DirtyElementMaxGridCellQueryCount;
	}


	// This function should be called with a dominant x direction only!
	// Todo: Refactor: Use TVectors consistently
	template <typename FunctionType>
	FORCEINLINE_DEBUGGABLE void DoForSweepIntersectCellsImp(const FReal QueryHalfExtentsX, const FReal QueryHalfExtentsY, const FReal StartPointX, const FReal StartPointY, const FReal RayX, const FReal RayY, FReal DirtyElementGridCellSize, FReal DirtyElementGridCellSizeInv, FunctionType InFunction)
	{
		// Use 2 paths (Line0 and Line1) that traces the shape of the swept AABB and fill between them
		// Example of one of the cases (XDirectionDominant, Fill Up):
		//                            Line0
		//                        #############
		//                      #             
		//                    #               
		//            Line0 #                 
		//                #                   #
		//              #                   #
		//            #                   #   ----> Dx
		//          #       ^           #
		//        #         |         # Line1
		//                 Fill     #
		//                  |     #
		//                      #
		//        #############
		//           Line1    ^TurningPointForLine1


		// The Reference Cell's origin is used for the local coordinates origin
		const int64 ReferenceCellIndexX = GetDirtyCellIndexFromWorldCoordinate(StartPointX, DirtyElementGridCellSizeInv);
		const int64 ReferenceCellIndexY = GetDirtyCellIndexFromWorldCoordinate(StartPointY, DirtyElementGridCellSizeInv);
		const TVector<FReal, 2> LocalCoordinatesOrigin{ (FReal)ReferenceCellIndexX * DirtyElementGridCellSize, (FReal)ReferenceCellIndexY * DirtyElementGridCellSize};
		const TVector<FReal, 2> StartPointLocal{ StartPointX - LocalCoordinatesOrigin.X, StartPointY - LocalCoordinatesOrigin.Y };
		const TVector<FReal, 2> EndPointLocal{ StartPointLocal.X + RayX, StartPointLocal.Y + RayY};

		FReal DeltaX = RayX;
		FReal DeltaY = RayY;

		FReal AbsDx = FMath::Abs(DeltaX);
		FReal AbsDy = FMath::Abs(DeltaY);

		bool DxTooSmall = AbsDx <= UE_SMALL_NUMBER;
		bool DyTooSmall = AbsDy <= UE_SMALL_NUMBER;

		int64 DeltaCelIndexX;
		int64 DeltaCelIndexY;
		FReal DtDy = 0; // DeltaTime over DeltaX
		FReal DtDx = 0;

		if (DxTooSmall)
		{
			// This is just the overlap case (no casting along a ray)
			DeltaCelIndexX = 1;
			DeltaCelIndexY = 1;
		}
		else
		{
			DeltaCelIndexX = DeltaX >= 0 ? 1 : -1;
			DeltaCelIndexY = DeltaY >= 0 ? 1 : -1;
		}

		// Use parametric description of the lines here (t is the parameter and is positive along the ray)
		// x = Dx/Dt*t + x0
		// y = Dy/Dt*t + x0
		DtDx = (FReal)DeltaCelIndexX;
		DtDy = DyTooSmall ? 1 : (FReal)DeltaCelIndexX * DeltaX / DeltaY;

		// Calculate all the bounds we need
		FReal XEndPointExpanded = EndPointLocal.X + (DeltaCelIndexX >= 0 ? QueryHalfExtentsX : -QueryHalfExtentsX);
		FReal YEndPointExpanded = EndPointLocal.Y + (DeltaCelIndexY >= 0 ? QueryHalfExtentsY : -QueryHalfExtentsY);
		FReal XStartPointExpanded = StartPointLocal.X + (DeltaCelIndexX >= 0 ? -QueryHalfExtentsX : QueryHalfExtentsX);
		FReal YStartPointExpanded = StartPointLocal.Y + (DeltaCelIndexY >= 0 ? -QueryHalfExtentsY : QueryHalfExtentsY);
		int64 TurningPointForLine1; // This is where we need to change direction for line 2
		TurningPointForLine1 = GetDirtyCellIndexFromWorldCoordinate(StartPointLocal.X + (DeltaCelIndexX >= 0 ? QueryHalfExtentsX : -QueryHalfExtentsX), DirtyElementGridCellSizeInv);

		// Line0 current position
		FReal X0 = XStartPointExpanded;
		FReal Y0 = YStartPointExpanded + QueryHalfExtentsY * (FReal)DeltaCelIndexY * 2;

		// Line1 current position
		FReal X1 = XStartPointExpanded;
		FReal Y1 = YStartPointExpanded;

		int64 CurrentCellIndexX0 = GetDirtyCellIndexFromWorldCoordinate(X0 + LocalCoordinatesOrigin.X, DirtyElementGridCellSizeInv);
		int64 CurrentCellIndexY0 = GetDirtyCellIndexFromWorldCoordinate(Y0 + LocalCoordinatesOrigin.Y, DirtyElementGridCellSizeInv);

		int64 CurrentCellIndexX1 = GetDirtyCellIndexFromWorldCoordinate(X1 + LocalCoordinatesOrigin.X, DirtyElementGridCellSizeInv);
		int64 CurrentCellIndexY1 = GetDirtyCellIndexFromWorldCoordinate(Y1 + LocalCoordinatesOrigin.Y, DirtyElementGridCellSizeInv);

		int64 LastCellIndexX = GetDirtyCellIndexFromWorldCoordinate(XEndPointExpanded + LocalCoordinatesOrigin.X, DirtyElementGridCellSizeInv);
		int64 LastCellIndexY = GetDirtyCellIndexFromWorldCoordinate(YEndPointExpanded + LocalCoordinatesOrigin.Y, DirtyElementGridCellSizeInv);

		// Because of floating point math precision there's case where we can get LastCellIndexY to be smaller than CurrentCellIndexY0 when the ray is stright down
		// that would cause the while loop to go on forever as it relies on an strict equality
		// we then need to adjust the value of LastCellIndexY accordingly
		if (DxTooSmall)
		{
			LastCellIndexY = FMath::Max(CurrentCellIndexY0, LastCellIndexY);
		}

		bool Done = false;
		while (!Done)
		{
			// Advance Line 0 crossing a horizontal border here (angle is 45 degrees or less)

			if (CurrentCellIndexY0 * DeltaCelIndexY < LastCellIndexY * DeltaCelIndexY && !DyTooSmall)
			{
				FReal CrossingVerticleCellBorderT = std::numeric_limits<FReal>::max();
				FReal CrossingHorizontalCellBorderT = std::numeric_limits<FReal>::max();
				CrossingVerticleCellBorderT = DtDx * ((FReal)(CurrentCellIndexX0 - ReferenceCellIndexX + (DeltaCelIndexX > 0 ? 1 : 0)) * DirtyElementGridCellSize - X0);
				CrossingHorizontalCellBorderT = DtDy * ((FReal)(CurrentCellIndexY0 - ReferenceCellIndexY + (DeltaCelIndexY > 0 ? 1 : 0)) * DirtyElementGridCellSize - Y0);
				if (CrossingHorizontalCellBorderT < CrossingVerticleCellBorderT)
				{
					X0 += CrossingHorizontalCellBorderT * (1 / DtDx);  // DtDx is always 1 or -1
					Y0 += CrossingHorizontalCellBorderT * (1 / DtDy);  // Abs(DtDy) >= 1
					CurrentCellIndexY0 += DeltaCelIndexY;
				}
			}

			for (int64 CurrentFillCellIndexY = CurrentCellIndexY1; CurrentFillCellIndexY * DeltaCelIndexY <= CurrentCellIndexY0 * DeltaCelIndexY; CurrentFillCellIndexY += DeltaCelIndexY)
			{
				InFunction((FReal)CurrentCellIndexX0 * DirtyElementGridCellSize, (FReal)CurrentFillCellIndexY * DirtyElementGridCellSize);
			}

			// Advance line 0 crossing vertical cell borders
			{
				if (CurrentCellIndexY0 != LastCellIndexY && !DyTooSmall)
				{
					FReal CrossingVerticleCellBorderT = std::numeric_limits<FReal>::max();
					CrossingVerticleCellBorderT = DtDx * ((FReal)(CurrentCellIndexX0 - ReferenceCellIndexX + (DeltaCelIndexX > 0 ? 1 : 0)) * DirtyElementGridCellSize - X0);
					X0 += CrossingVerticleCellBorderT * (1 / DtDx);
					Y0 += CrossingVerticleCellBorderT * (1 / DtDy);
				}
				else
				{
					X0 += DirtyElementGridCellSize * (FReal)DeltaCelIndexX;
				}
			}

			// Advance line 1
			if (CurrentCellIndexX1 != LastCellIndexX)
			{
				if ((CurrentCellIndexX1 - ReferenceCellIndexX) * DeltaCelIndexX < TurningPointForLine1 * DeltaCelIndexX)
				{
					X1 += DirtyElementGridCellSize * (FReal)DeltaCelIndexX;
				}
				else
				{
					if ((CurrentCellIndexX1 - ReferenceCellIndexX) == TurningPointForLine1)
					{
						// Put Line position exactly at the turning point
						X1 = StartPointLocal.X + (DeltaCelIndexX >= 0 ? QueryHalfExtentsX : -QueryHalfExtentsX);
					}

					FReal CrossingVerticleCellBorderT = std::numeric_limits<FReal>::max();
					FReal CrossingHorizontalCellBorderT = std::numeric_limits<FReal>::max();
					if (!DxTooSmall)
					{
						CrossingVerticleCellBorderT = DtDx * ((FReal)(CurrentCellIndexX1 - ReferenceCellIndexX + (DeltaCelIndexX > 0 ? 1 : 0)) * DirtyElementGridCellSize - X1);
					}

					if (!DyTooSmall)
					{
						CrossingHorizontalCellBorderT = DtDy * ((FReal)(CurrentCellIndexY1 - ReferenceCellIndexY + (DeltaCelIndexY > 0 ? 1 : 0)) * DirtyElementGridCellSize - Y1);
					}

					if (CrossingHorizontalCellBorderT < CrossingVerticleCellBorderT)
					{
						CurrentCellIndexY1 += DeltaCelIndexY;
					}

					if (!DxTooSmall)
					{
						X1 += CrossingVerticleCellBorderT * (1 / DtDx);
					}

					if (!DyTooSmall)
					{
						Y1 += CrossingVerticleCellBorderT * (1 / DtDy);
					}
				}
				CurrentCellIndexX1 += DeltaCelIndexX;
			}

			CurrentCellIndexX0 += DeltaCelIndexX;
			Done = (CurrentCellIndexY0 == LastCellIndexY) && (DeltaCelIndexX * CurrentCellIndexX0 > LastCellIndexX * DeltaCelIndexX);
		}
	}

	template <typename FunctionType>
	void DoForSweepIntersectCells(const FVec3 QueryHalfExtents, const FVec3& StartPoint, const FVec3& Dir, FReal Length, FReal DirtyElementGridCellSize, FReal DirtyElementGridCellSizeInv, FunctionType InFunction)
	{
		FReal AbsDx = FMath::Abs(Dir.X * Length);
		FReal AbsDy = FMath::Abs(Dir.Y * Length);

		bool XDirectionDominant = AbsDx >= AbsDy;

		// if the ray is mostly vertical then we can default to an simple overlap 
		if (AbsDx <= UE_SMALL_NUMBER && AbsDy <= UE_SMALL_NUMBER)
		{
			// no need to account for the ray length as we only collect 2 dimensional cell coordinates and the ray is already proven to be vertical
			const FAABB3 QueryBounds(StartPoint- QueryHalfExtents, StartPoint + QueryHalfExtents);

			const int64 CellStartX = GetDirtyCellIndexFromWorldCoordinate(QueryBounds.Min().X, DirtyElementGridCellSizeInv);
			const int64 CellStartY = GetDirtyCellIndexFromWorldCoordinate(QueryBounds.Min().Y, DirtyElementGridCellSizeInv);
			const int64 CellEndX = GetDirtyCellIndexFromWorldCoordinate(QueryBounds.Max().X, DirtyElementGridCellSizeInv);
			const int64 CellEndY = GetDirtyCellIndexFromWorldCoordinate(QueryBounds.Max().Y, DirtyElementGridCellSizeInv);
			
			for (int64 X = CellStartX; X <= CellEndX; X++)
			{
				for (int64 Y = CellStartY; Y <= CellEndY; Y++)
				{
					InFunction((FReal)X * DirtyElementGridCellSize, (FReal)Y * DirtyElementGridCellSize);
				}
			}
		}
		else
		{
			if (XDirectionDominant)
			{
				DoForSweepIntersectCellsImp(QueryHalfExtents.X, QueryHalfExtents.Y, StartPoint.X, StartPoint.Y, Dir.X * Length, Dir.Y * Length, DirtyElementGridCellSize, DirtyElementGridCellSizeInv, InFunction);
			}
			else
			{
				// Swap Y and X
				DoForSweepIntersectCellsImp(QueryHalfExtents.Y, QueryHalfExtents.X, StartPoint.Y, StartPoint.X, Dir.Y * Length, Dir.X * Length, DirtyElementGridCellSize, DirtyElementGridCellSizeInv, [&](auto X, auto Y) {InFunction(Y, X); });
			}
		}

	}

	FORCEINLINE_DEBUGGABLE bool TooManyRaycastQueryCells(const FVec3& StartPoint, const FVec3& Dir, const FReal Length, FReal DirtyElementGridCellSizeInv, int32 DirtyElementMaxGridCellQueryCount)
	{
		if(Length > (FReal)TNumericLimits<uint64>::Max() || DirtyElementGridCellSizeInv == 0)
		{
			return true;
		}

		FVec3 EndPoint = StartPoint + Length * Dir;

		const FReal ElementAbsLimit = (FReal)TNumericLimits<int64>::Max() / DirtyElementGridCellSizeInv;
		if(StartPoint.GetAbsMax() >= ElementAbsLimit || EndPoint.GetAbsMax() >= ElementAbsLimit)
		{
			return true;
		}

		int64 FirstCellIndexX = GetDirtyCellIndexFromWorldCoordinate(StartPoint.X, DirtyElementGridCellSizeInv);
		int64 FirstCellIndexY = GetDirtyCellIndexFromWorldCoordinate(StartPoint.Y, DirtyElementGridCellSizeInv);

		int64 LastCellIndexX = GetDirtyCellIndexFromWorldCoordinate(EndPoint.X, DirtyElementGridCellSizeInv);
		int64 LastCellIndexY = GetDirtyCellIndexFromWorldCoordinate(EndPoint.Y, DirtyElementGridCellSizeInv);

		// This will be equal to the Manhattan distance 
		uint64 CellCount = FMath::Abs(FirstCellIndexX - LastCellIndexX) + FMath::Abs(FirstCellIndexY - LastCellIndexY);

		if (CellCount > (uint64)DirtyElementMaxGridCellQueryCount)
		{
			return true;
		}

		return false;
	}

	template <typename FunctionType>
	FORCEINLINE_DEBUGGABLE void DoForRaycastIntersectCells(const FVec3& StartPoint, const FVec3& Dir, FReal Length, FReal DirtyElementGridCellSize, FReal DirtyElementGridCellSizeInv, FunctionType InFunction)
	{
		const int64 FirstCellIndexX = GetDirtyCellIndexFromWorldCoordinate(StartPoint.X, DirtyElementGridCellSizeInv);
		const int64 FirstCellIndexY = GetDirtyCellIndexFromWorldCoordinate(StartPoint.Y, DirtyElementGridCellSizeInv);

		int64 CurrentCellIndexX = FirstCellIndexX;
		int64 CurrentCellIndexY = FirstCellIndexY;

		// Note: local coordinates are relative to the StartPoint cell origin
		const FVec3 LocalCoordinatesOrigin{ (FReal)CurrentCellIndexX * DirtyElementGridCellSize, (FReal)CurrentCellIndexY * DirtyElementGridCellSize,  StartPoint.Z};

		const FVec3 StartPointLocal = StartPoint - LocalCoordinatesOrigin;

		const FVec3 EndPointLocal = StartPointLocal + Length * Dir; // Local coordinates are relative to the start point
		const FVec3 EndPoint = LocalCoordinatesOrigin + EndPointLocal;
		int64 LastCellIndexX = GetDirtyCellIndexFromWorldCoordinate(EndPoint.X, DirtyElementGridCellSizeInv);
		int64 LastCellIndexY = GetDirtyCellIndexFromWorldCoordinate(EndPoint.Y, DirtyElementGridCellSizeInv);

		FReal DeltaX = EndPointLocal.X - StartPointLocal.X;
		FReal DeltaY = EndPointLocal.Y - StartPointLocal.Y;

		FReal AbsDx = FMath::Abs(DeltaX);
		FReal AbsDy = FMath::Abs(DeltaY);

		bool DxTooSmall = AbsDx <= UE_SMALL_NUMBER;
		bool DyTooSmall = AbsDy <= UE_SMALL_NUMBER;

		if (DxTooSmall && DyTooSmall)
		{
			InFunction(HashCoordinates(StartPoint.X, StartPoint.Y, DirtyElementGridCellSizeInv));
			return;
		}

		int DeltaCelIndexX = DeltaX >= 0 ? 1 : -1;
		int DeltaCelIndexY = DeltaY >= 0 ? 1 : -1;

		FReal DtDy = 0; // DeltaTime over DeltaX
		FReal DtDx = 0;

		bool XDirectionDominant = AbsDx >= AbsDy;
		// Use parametric description of line here (t is the parameter and is positive along the ray)
		// x = Dx/Dt*t + x0
		// y = Dy/Dt*t + x0
		if (XDirectionDominant)
		{
			DtDx = (FReal)DeltaCelIndexX;
			DtDy = DyTooSmall ? 1 : (FReal)DeltaCelIndexX * DeltaX / DeltaY;
		}
		else
		{
			DtDx = DxTooSmall ? 1 : (FReal)DeltaCelIndexY * DeltaY / DeltaX;
			DtDy = (FReal)DeltaCelIndexY;
		}

		// These are local coordinates
		FReal X = StartPointLocal.X;
		FReal Y = StartPointLocal.Y;

		bool Done = false;
		while (!Done)
		{
			InFunction(HashCell(CurrentCellIndexX, CurrentCellIndexY));
			FReal CrossingVerticleCellBorderT = std::numeric_limits<FReal>::max();
			FReal CrossingHorizontalCellBorderT = std::numeric_limits<FReal>::max();
			if (!DxTooSmall)
			{
				CrossingVerticleCellBorderT = DtDx * ((FReal)(CurrentCellIndexX - FirstCellIndexX + (DeltaCelIndexX > 0 ? 1 : 0)) * DirtyElementGridCellSize - X);
			}

			if (!DyTooSmall)
			{
				CrossingHorizontalCellBorderT = DtDy * ((FReal)(CurrentCellIndexY - FirstCellIndexY + (DeltaCelIndexY > 0 ? 1 : 0)) * DirtyElementGridCellSize - Y);
			}

			FReal SmallestT;
			if (CrossingVerticleCellBorderT <= CrossingHorizontalCellBorderT)
			{
				CurrentCellIndexX += DeltaCelIndexX;
				SmallestT = CrossingVerticleCellBorderT;
			}
			else
			{
				CurrentCellIndexY += DeltaCelIndexY;
				SmallestT = CrossingHorizontalCellBorderT;
			}

			if (!DxTooSmall)
			{
				X += SmallestT * (1 / DtDx);
			}

			if (!DyTooSmall)
			{
				Y += SmallestT * (1 / DtDy);
			}

			if (DeltaCelIndexX * CurrentCellIndexX > DeltaCelIndexX * LastCellIndexX || DeltaCelIndexY * CurrentCellIndexY > DeltaCelIndexY * LastCellIndexY)
			{
				Done = true;
			}
		}
	}
}
