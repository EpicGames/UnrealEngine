// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/AABB.h"
#include "Chaos/Core.h"
#include "ChaosArchive.h"
#include "Math/VectorRegister.h"
#include "Chaos/VectorUtility.h"


namespace Chaos
{

	class alignas(16) FAABBVectorized
	{
	public:

		FORCEINLINE_DEBUGGABLE FAABBVectorized()
		{
			Min = GlobalVectorConstants::BigNumber;
			Max = VectorNegate(GlobalVectorConstants::BigNumber);
		}

		FORCEINLINE_DEBUGGABLE FAABBVectorized(const VectorRegister4Float& InMin, const VectorRegister4Float& InMax)
			: Min(InMin)
			, Max(InMax)
		{
		}
		template<typename T>
		FORCEINLINE_DEBUGGABLE explicit FAABBVectorized(const TAABB<T, 3>& AABB)
		{
			const VectorRegister4Float MinSmid = MakeVectorRegisterFloatFromDouble(MakeVectorRegisterDouble(AABB.Min().X, AABB.Min().Y, AABB.Min().Z, 0.0));
			const VectorRegister4Float MaxSmid = MakeVectorRegisterFloatFromDouble(MakeVectorRegisterDouble(AABB.Max().X, AABB.Max().Y, AABB.Max().Z, 0.0));
			Min = MinSmid;
			Max = MaxSmid;
		}

		FORCEINLINE_DEBUGGABLE const VectorRegister4Float& GetMin() const { return Min; }
		FORCEINLINE_DEBUGGABLE const VectorRegister4Float& GetMax() const { return Max; }

		FORCEINLINE_DEBUGGABLE void SetMin(const VectorRegister4Float& InMin) { Min = InMin; }
		FORCEINLINE_DEBUGGABLE void SetMax(const VectorRegister4Float& InMax) { Max = InMax; }

		FORCEINLINE_DEBUGGABLE bool RaycastFast(const VectorRegister4Float& StartPoint, const VectorRegister4Float& InvDir, const VectorRegister4Float& Parallel, const VectorRegister4Float& Length) const
		{
			const VectorRegister4Float StarToMinGTZero = VectorCompareGT(Min, StartPoint);
			const VectorRegister4Float ZeroGTStarToMax = VectorCompareGT(StartPoint, Max);
			VectorRegister4Float IsFalse = VectorBitwiseAnd(VectorBitwiseOr(StarToMinGTZero, ZeroGTStarToMax), Parallel);

			if (::VectorMaskBits(IsFalse))
			{
				return false;	//parallel and outside
			}

			const VectorRegister4Float StartToMin = VectorSubtract(Min, StartPoint);
			const VectorRegister4Float StartToMax = VectorSubtract(Max, StartPoint);
			const VectorRegister4Float StartToMinInvDir = VectorMultiply(StartToMin, InvDir);
			const VectorRegister4Float StartToMaxInvDir = VectorMultiply(StartToMax, InvDir);

			const VectorRegister4Float Time1 = VectorBitwiseNotAnd(Parallel, StartToMinInvDir);
			const VectorRegister4Float Time2 = VectorSelect(Parallel, Length, StartToMaxInvDir);

			const VectorRegister4Float SortedTime1 = VectorMin(Time1, Time2);
			const VectorRegister4Float SortedTime2 = VectorMax(Time1, Time2);

			VectorRegister4Float LatestStartTime = VectorMax(SortedTime1, VectorSwizzle(SortedTime1, 1, 2, 0, 3));
			LatestStartTime = VectorMax(LatestStartTime, VectorSwizzle(SortedTime1, 2, 0, 1, 3));
			LatestStartTime = VectorMax(LatestStartTime, VectorZero());

			VectorRegister4Float EarliestEndTime = VectorMin(SortedTime2, VectorSwizzle(SortedTime2, 1, 2, 0, 3));
			EarliestEndTime = VectorMin(EarliestEndTime, VectorSwizzle(SortedTime2, 2, 0, 1, 3));
			EarliestEndTime = VectorMin(EarliestEndTime, Length);

			//Outside of slab before entering another
			IsFalse = VectorCompareGT(LatestStartTime, EarliestEndTime);

			return ::VectorMaskBits(IsFalse) == 0;
		}

		FORCEINLINE_DEBUGGABLE bool Intersects(const FAABBVectorized& Other) const
		{
			VectorRegister4Float IsFalse = VectorBitwiseOr(VectorCompareGT(Min, Other.GetMax()), VectorCompareGT(Other.GetMin(), Max));

			return !static_cast<bool>(::VectorMaskBits(IsFalse));
		}

		FORCEINLINE_DEBUGGABLE FAABBVectorized& Thicken(const VectorRegister4Float& Thickness)
		{
			Min = VectorSubtract(Min, Thickness);
			Max = VectorAdd(Max, Thickness);
			return *this;
		}

	private:
		VectorRegister4Float Min;
		VectorRegister4Float Max;
	};
}