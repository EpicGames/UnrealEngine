// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/AABB.h"
#include "Chaos/Core.h"
#include "ChaosArchive.h"
#include "Math/VectorRegister.h"
#include "Chaos/VectorUtility.h"


namespace Chaos
{

	class alignas(32) FAABBVectorizedDouble
	{
	public:

		FORCEINLINE_DEBUGGABLE FAABBVectorizedDouble()
		{
			Min = GlobalVectorConstants::DoubleBigNumber;
			Max = VectorNegate(GlobalVectorConstants::DoubleBigNumber);
		}

		FORCEINLINE_DEBUGGABLE FAABBVectorizedDouble(const VectorRegister4Double& InMin, const VectorRegister4Double& InMax)
			: Min(InMin)
			, Max(InMax)
		{
		}
		template<typename T>
		FORCEINLINE_DEBUGGABLE explicit FAABBVectorizedDouble(const TAABB<T, 3>& AABB)
		{
			Min = MakeVectorRegisterDouble(AABB.Min().X, AABB.Min().Y, AABB.Min().Z, 0.0);
			Max = MakeVectorRegisterDouble(AABB.Max().X, AABB.Max().Y, AABB.Max().Z, 0.0);
		}

		FORCEINLINE_DEBUGGABLE const VectorRegister4Double& GetMin() const { return Min; }
		FORCEINLINE_DEBUGGABLE const VectorRegister4Double& GetMax() const { return Max; }

		FORCEINLINE_DEBUGGABLE void SetMin(const VectorRegister4Double& InMin) { Min = InMin; }
		FORCEINLINE_DEBUGGABLE void SetMax(const VectorRegister4Double& InMax) { Max = InMax; }

		FORCEINLINE_DEBUGGABLE bool RaycastFast(const VectorRegister4Double& StartPoint, const VectorRegister4Double& InvDir, const VectorRegister4Double& Parallel, const VectorRegister4Double& Length, VectorRegister4Double& LatestStartTime) const
		{
			const VectorRegister4Double StarToMinGTZero = VectorCompareGT(Min, StartPoint);
			const VectorRegister4Double ZeroGTStarToMax = VectorCompareGT(StartPoint, Max);
			VectorRegister4Double IsFalse = VectorBitwiseAnd(VectorBitwiseOr(StarToMinGTZero, ZeroGTStarToMax), Parallel);

			if (VectorMaskBits(IsFalse))
			{
				return false;	//parallel and outside
			}

			const VectorRegister4Double StartToMin = VectorSubtract(Min, StartPoint);
			const VectorRegister4Double StartToMax = VectorSubtract(Max, StartPoint);
			const VectorRegister4Double StartToMinInvDir = VectorMultiply(StartToMin, InvDir);
			const VectorRegister4Double StartToMaxInvDir = VectorMultiply(StartToMax, InvDir);

			const VectorRegister4Double Time1 = VectorBitwiseNotAnd(Parallel, StartToMinInvDir);
			const VectorRegister4Double Time2 = VectorSelect(Parallel, Length, StartToMaxInvDir);

			const VectorRegister4Double SortedTime1 = VectorMin(Time1, Time2);
			const VectorRegister4Double SortedTime2 = VectorMax(Time1, Time2);

			LatestStartTime = VectorMax(SortedTime1, VectorSwizzle(SortedTime1, 1, 2, 0, 3));
			LatestStartTime = VectorMax(LatestStartTime, VectorSwizzle(SortedTime1, 2, 0, 1, 3));
			LatestStartTime = VectorMax(LatestStartTime, VectorZero());

			VectorRegister4Double EarliestEndTime = VectorMin(SortedTime2, VectorSwizzle(SortedTime2, 1, 2, 0, 3));
			EarliestEndTime = VectorMin(EarliestEndTime, VectorSwizzle(SortedTime2, 2, 0, 1, 3));
			EarliestEndTime = VectorMin(EarliestEndTime, Length);

			//Outside of slab before entering another
			IsFalse = VectorCompareGT(LatestStartTime, EarliestEndTime);

			return VectorMaskBits(IsFalse) == 0;
		}

		FORCEINLINE_DEBUGGABLE bool Intersects(const FAABBVectorizedDouble& Other) const
		{
			VectorRegister4Double IsFalse = VectorBitwiseOr(VectorCompareGT(Min, Other.GetMax()), VectorCompareGT(Other.GetMin(), Max));

			return !static_cast<bool>(VectorMaskBits(IsFalse));
		}

		FORCEINLINE_DEBUGGABLE FAABBVectorizedDouble& Thicken(const VectorRegister4Double& Thickness)
		{
			Min = VectorSubtract(Min, Thickness);
			Max = VectorAdd(Max, Thickness);
			return *this;
		}

	private:
		VectorRegister4Double Min;
		VectorRegister4Double Max;
	};
}