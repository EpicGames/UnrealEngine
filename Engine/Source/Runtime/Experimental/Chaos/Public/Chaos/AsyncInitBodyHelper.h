// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/ScopeExit.h"

namespace Chaos
{
	namespace CVars
	{
		extern CHAOS_API bool bEnableAsyncInitBody;
	}

	namespace Private
	{
		template<typename T>
		struct FChaosAsyncInitBodyWriteScopeLock
		{
			inline FChaosAsyncInitBodyWriteScopeLock(T& InLock)
				: Lock(Chaos::CVars::bEnableAsyncInitBody ? &InLock : nullptr)
			{
				if (Lock)
				{
					Lock->WriteLock();
				}
			}

			inline ~FChaosAsyncInitBodyWriteScopeLock()
			{
				if (Lock)
				{
					Lock->WriteUnlock();
				}
			}

			T* Lock;
		};

		template<typename T>
		struct FChaosAsyncInitBodyReadScopeLock
		{
			inline FChaosAsyncInitBodyReadScopeLock(T& InLock)
				: Lock(Chaos::CVars::bEnableAsyncInitBody ? &InLock : nullptr)
			{
				if (Lock)
				{
					Lock->ReadLock();
				}
			}

			inline ~FChaosAsyncInitBodyReadScopeLock()
			{
				if (Lock)
				{
					Lock->ReadUnlock();
				}
			}

			T* Lock;
		};
	}
}

#define UE_CHAOS_ASYNC_INITBODY_WRITESCOPELOCK(x) Chaos::Private::FChaosAsyncInitBodyWriteScopeLock WriteScopeLock(x)
#define UE_CHAOS_ASYNC_INITBODY_READSCOPELOCK(x)  Chaos::Private::FChaosAsyncInitBodyReadScopeLock ReadScopeLock(x)
