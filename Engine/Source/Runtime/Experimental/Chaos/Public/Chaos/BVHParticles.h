// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Particles.h"
#include "ChaosArchive.h"
#include "Chaos/ParticleHandleFwd.h"


namespace Chaos
{
	extern int32 CHAOS_API CollisionParticlesBVHDepth;

    template<class OBJECT_ARRAY, class LEAF_TYPE, class T, int d>
    class TBoundingVolumeHierarchy;


    template<class OBJECT_ARRAY, class LEAF_TYPE, class T, int d>
    class TBoundingVolumeHierarchy;

    template<class T, int d>
    class TAABB;

	class FBVHParticles final /*Note: removing this final has implications for serialization. See TImplicitObject*/ : public FParticles
	{
	public:
		using TArrayCollection::Size;
		using FParticles::X;
		using FParticles::AddParticles;

		CHAOS_API FBVHParticles();
		FBVHParticles(FBVHParticles&& Other);
		FBVHParticles(FParticles&& Other);
        CHAOS_API ~FBVHParticles();

	    CHAOS_API FBVHParticles& operator=(const FBVHParticles& Other);
	    CHAOS_API FBVHParticles& operator=(FBVHParticles&& Other);

		FBVHParticles* NewCopy() const
		{
			return new FBVHParticles(*this);
		}

		CHAOS_API void UpdateAccelerationStructures();
		const TArray<int32> FindAllIntersections(const FAABB3& Object) const;

		static FBVHParticles* SerializationFactory(FChaosArchive& Ar, FBVHParticles* BVHParticles)
		{
			return Ar.IsLoading() ? new FBVHParticles() : nullptr;
		}

		CHAOS_API void Serialize(FChaosArchive& Ar);

		void Serialize(FArchive& Ar)
		{
			check(false); //Aggregate simplicial require FChaosArchive - check false by default
		}

	private:
		CHAOS_API FBVHParticles(const FBVHParticles& Other);

		TBoundingVolumeHierarchy<FParticles, TArray<int32>, FReal, 3>* MBVH;
	};

	FORCEINLINE FChaosArchive& operator<<(FChaosArchive& Ar, FBVHParticles& Value)
	{
		Value.Serialize(Ar);
		return Ar;
	}

	FORCEINLINE FArchive& operator<<(FArchive& Ar, FBVHParticles& Value)
	{
		Value.Serialize(Ar);
		return Ar;
	}


	template<class T, int d>
	using TBVHParticles = FBVHParticles;

	typedef TBVHParticles<FReal, 3> FBVHParticlesFloat3;
}
