// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/PBDSpringConstraintsBase.h"
#include "ChaosStats.h"
#include "Chaos/ImplicitQRSVD.h"
#include "Chaos/GraphColoring.h"
#include "Chaos/Framework/Parallel.h"
#include "Chaos/XPBDCorotatedConstraints.h"


namespace Chaos::Softs
{
	class FBlendedXPBDCorotatedTetrahedralConstraints : public FXPBDCorotatedTetrahedralConstraints
	{

		typedef FXPBDCorotatedTetrahedralConstraints Base;
		using Base::MeshConstraints;
		using Base::LambdaArray;
		using Base::Measure;
		using Base::DmInverse;
		using Base::Lambda;
		using Base::bRecordMetric;

	public:
		//this one only accepts tetmesh input and mesh
		FBlendedXPBDCorotatedTetrahedralConstraints(
			const FSolverParticles& InParticles,
			const TArray<TVector<int32, 4>>& InMesh,
			const bool bRecordMetricIn = true,
			const FSolverReal& EMesh = (FSolverReal)10.0,
			const FSolverReal& NuMesh = (FSolverReal).3,
			const FSolverReal& InZeta = (FSolverReal)1.
		)
			: Base(InParticles, InMesh, bRecordMetricIn, EMesh, NuMesh), Zeta(InZeta)
		{
			if (Zeta < (FSolverReal)0.) 
			{
				Zeta = (FSolverReal)0.;
			}
			if (Zeta > (FSolverReal)1.) 
			{
				Zeta = (FSolverReal)1.;
			}

			C1Contribution.Init(TVec4<TVector<FSolverReal, 3>>(TVector<FSolverReal, 3>((FSolverReal)0.)), InMesh.Num());
			C2Contribution.Init(TVec4<TVector<FSolverReal, 3>>(TVector<FSolverReal, 3>((FSolverReal)0.)), InMesh.Num());
		}

		//this one only accepts tetmesh input and mesh
		FBlendedXPBDCorotatedTetrahedralConstraints(
			const FSolverParticles& InParticles,
			const TArray<TVector<int32, 4>>& InMesh,
			const TArray<FSolverReal>& EMeshArray,
			const FSolverReal& NuMesh = (FSolverReal).3,
			const bool bRecordMetricIn = false,
			const FSolverReal& InZeta = (FSolverReal)1.
		)
			: Base(InParticles, InMesh, EMeshArray, NuMesh, bRecordMetricIn), Zeta(InZeta)
		{
			if (Zeta < (FSolverReal)0.)
			{
				Zeta = (FSolverReal)0.;
			}
			if (Zeta > (FSolverReal)1.)
			{
				Zeta = (FSolverReal)1.;
			}

			C1Contribution.Init(TVec4<TVector<FSolverReal, 3>>(TVector<FSolverReal, 3>((FSolverReal)0.)), InMesh.Num());
			C2Contribution.Init(TVec4<TVector<FSolverReal, 3>>(TVector<FSolverReal, 3>((FSolverReal)0.)), InMesh.Num());
		}

		virtual ~FBlendedXPBDCorotatedTetrahedralConstraints() = default;

		virtual void ApplyInSerial(FSolverParticlesRange& Particles, const FSolverReal Dt, const int32 ElementIndex) const override
		{
			TVec4<TVector<FSolverReal, 3>> PolarDelta = GetPolarDelta(Particles, Dt, ElementIndex);

			for (int i = 0; i < 4; i++)
			{
				Particles.P(MeshConstraints[ElementIndex][i]) += PolarDelta[i];
			}

			TVec4<TVector<FSolverReal, 3>> DetDelta = GetDeterminantDelta(Particles, Dt, ElementIndex);

			for (int i = 0; i < 4; i++)
			{
				Particles.P(MeshConstraints[ElementIndex][i]) += DetDelta[i];
			}
		}

		virtual void Init() const override
		{
			for (FSolverReal& Lambdas : LambdaArray) { Lambdas = (FSolverReal)0.; }
			C1Contribution.Init(TVec4<TVector<FSolverReal, 3>>(TVector<FSolverReal, 3>((FSolverReal)0.)), MeshConstraints.Num());
			C2Contribution.Init(TVec4<TVector<FSolverReal, 3>>(TVector<FSolverReal, 3>((FSolverReal)0.)), MeshConstraints.Num());
		}

	protected:
		virtual TVec4<TVector<FSolverReal, 3>> GetDeterminantDelta(const FSolverParticlesRange& Particles, const FSolverReal Dt, const int32 ElementIndex, const FSolverReal Tol = (FSolverReal)1e-3) const override
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(STAT_ChaosBlendedXPBDCorotatedApplyDet);

			const PMatrix<FSolverReal, 3, 3> Fe = Base::F(ElementIndex, Particles);
			PMatrix<FSolverReal, 3, 3> DmInvT = Base::ElementDmInv(ElementIndex).GetTransposed();

			FSolverReal J = Fe.Determinant();
			if (J - (FSolverReal)1. < Tol)
			{
				return TVec4<TVector<FSolverReal, 3>>(TVector<FSolverReal, 3>((FSolverReal)0.));
			}

			TVec4<TVector<FSolverReal, 3>> dC2 = Base::GetDeterminantGradient(Fe, DmInvT);

			FSolverReal AlphaTilde = (FSolverReal)2. / (Dt * Dt * Base::Lambda * Measure[ElementIndex]);

			FSolverReal DLambda = (1 - J) - AlphaTilde * LambdaArray[2 * ElementIndex + 1];

			FSolverReal Denom = AlphaTilde;
			for (int i = 0; i < 4; i++)
			{
				for (int j = 0; j < 3; j++)
				{
					Denom += dC2[i][j] * Particles.InvM(MeshConstraints[ElementIndex][i]) * dC2[i][j];
				}
			}
			DLambda /= Denom;
			LambdaArray[2 * ElementIndex + 1] += DLambda;
			TVec4<TVector<FSolverReal, 3>> Delta2(TVector<FSolverReal, 3>((FSolverReal)0.));
			for (int i = 0; i < 4; i++)
			{
				for (int j = 0; j < 3; j++)
				{
					Delta2[i][j] = Particles.InvM(MeshConstraints[ElementIndex][i]) * dC2[i][j] * DLambda;
				}
			}
			TVec4<TVector<FSolverReal, 3>> Delta1(TVector<FSolverReal, 3>((FSolverReal)0.));
			for (int i = 0; i < 4; i++)
			{
				for (int j = 0; j < 3; j++)
				{
					Delta1[i][j] = Particles.InvM(MeshConstraints[ElementIndex][i]) 
						* dC2[i][j] * LambdaArray[2 * ElementIndex + 1] - C2Contribution[ElementIndex][i][j];
				}
			}
			TVec4<TVector<FSolverReal, 3>> Delta(TVector<FSolverReal, 3>((FSolverReal)0.));
			for (int i = 0; i < 4; i++)
			{
				Delta[i] = Zeta * Delta1[i] + ((FSolverReal)1. - Zeta) * Delta2[i];
			}

			for (int i = 0; i < 4; i++)
			{
				C2Contribution[ElementIndex][i] += Delta[i];
			}

			return Delta;
		}

		virtual TVec4<TVector<FSolverReal, 3>> GetPolarDelta(const FSolverParticlesRange& Particles, const FSolverReal Dt, const int32 ElementIndex, const FSolverReal Tol = (FSolverReal)1e-3) const override
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(STAT_ChaosBlendedXPBDCorotatedApplyPolar);
			SCOPE_CYCLE_COUNTER(STAT_ChaosXPBDCorotatedPolar);
			const PMatrix<FSolverReal, 3, 3> Fe = Base::F(ElementIndex, Particles);

			PMatrix<FSolverReal, 3, 3> Re((FSolverReal)0.), Se((FSolverReal)0.);

			Chaos::PolarDecomposition(Fe, Re, Se);

			FSolverReal C1 = (FSolverReal)0.;
			for (int i = 0; i < 3; i++)
			{
				for (int j = 0; j < 3; j++)
				{
					C1 += FMath::Square((Fe - Re).GetAt(i, j));
				}
			}
			C1 = FMath::Sqrt(C1);

			if (C1 < Tol)
			{
				return TVec4<TVector<FSolverReal, 3>>(TVector<FSolverReal, 3>((FSolverReal)0.));
			}

			PMatrix<FSolverReal, 3, 3> DmInvT = Base::ElementDmInv(ElementIndex).GetTransposed();

			TVec4<TVector<FSolverReal, 3>> dC1 = Base::GetPolarGradient(Fe, Re, DmInvT, C1);

			FSolverReal AlphaTilde = (FSolverReal)1. / (Dt * Dt * Base::Mu * Measure[ElementIndex]);

			FSolverReal DLambda = -C1 - AlphaTilde * LambdaArray[2 * ElementIndex + 0];

			FSolverReal Denom = AlphaTilde;
			for (int i = 0; i < 4; i++)
			{
				for (int j = 0; j < 3; j++)
				{
					Denom += dC1[i][j] * Particles.InvM(MeshConstraints[ElementIndex][i]) * dC1[i][j];
				}
			}
			DLambda /= Denom;
			LambdaArray[2 * ElementIndex + 0] += DLambda;

			TVec4<TVector<FSolverReal, 3>> Delta2(TVector<FSolverReal, 3>((FSolverReal)0.));
			for (int i = 0; i < 4; i++)
			{
				for (int j = 0; j < 3; j++)
				{
					Delta2[i][j] = Particles.InvM(MeshConstraints[ElementIndex][i]) * dC1[i][j] * DLambda;
				}
			}
			TVec4<TVector<FSolverReal, 3>> Delta1(TVector<FSolverReal, 3>((FSolverReal)0.));
			for (int i = 0; i < 4; i++)
			{
				for (int j = 0; j < 3; j++)
				{
					Delta1[i][j] = Particles.InvM(MeshConstraints[ElementIndex][i])
						* dC1[i][j] * LambdaArray[2 * ElementIndex + 0] - C1Contribution[ElementIndex][i][j];
				}
			}
			TVec4<TVector<FSolverReal, 3>> Delta(TVector<FSolverReal, 3>((FSolverReal)0.));
			for (int i = 0; i < 4; i++)
			{
				Delta[i] = Zeta * Delta1[i] + ((FSolverReal)1. - Zeta) * Delta2[i];
			}

			for (int i = 0; i < 4; i++)
			{
				C1Contribution[ElementIndex][i] += Delta[i];
			}

			return Delta;

		}


	private:

		FSolverReal Zeta;
		mutable TArray<TVec4<TVector<FSolverReal, 3>>> C1Contribution;
		mutable TArray<TVec4<TVector<FSolverReal, 3>>> C2Contribution;
	};

	template <typename T, typename ParticleType>
	using FBlendedXPBDCorotatedConstraints UE_DEPRECATED(5.8, "Deprecated. this class is to be deleted, use FBlendedXPBDCorotatedTetrahedralConstraints instead") = FBlendedXPBDCorotatedTetrahedralConstraints;

}  // End namespace Chaos::Softs

