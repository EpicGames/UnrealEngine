// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Real.h"
#include "Chaos/Array.h"
#include "Chaos/ArrayND.h"
#include "Chaos/BoundingVolumeUtilities.h"
#include "Chaos/Box.h"
#include "Chaos/Defines.h"
#include "Chaos/Framework/Parallel.h"
#include "Chaos/Transform.h"
#include "Chaos/UniformGrid.h"
#include "Chaos/ISpatialAcceleration.h"
#include "ChaosStats.h"
#include "ChaosLog.h"
#include "HAL/IConsoleManager.h"
#include "Chaos/ParticleHandleFwd.h"
#include "Templates/Models.h"

#include <memory>
#include <unordered_set>

// Required for debug blocks below in raycasts
//#include "Engine/Engine.h"
//#include "Engine/World.h"
//#include "DrawDebugHelpers.h"

class FChaosVDDataWrapperUtils;

template <typename T, bool>
struct TSpatialAccelerationTraits
{
};

template <typename TSOA>
struct TSpatialAccelerationTraits<TSOA, true>
{
	using TPayloadType = typename TSOA::THandleType;
};

template <typename T>
struct TSpatialAccelerationTraits<T, false>
{
	using TPayloadType = int32;
};

struct CParticleView
{
	template <typename T>
	auto Requires(typename T::THandleType) ->void;
};

struct FBoundingVolumeCVars
{
	static int32 FilterFarBodies;
	static FAutoConsoleVariableRef CVarFilterFarBodies;
};

DECLARE_CYCLE_STAT(TEXT("BoundingVolumeGenerateTree"), STAT_BoundingVolumeGenerateTree, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("BoundingVolumeComputeGlobalBox"), STAT_BoundingVolumeComputeGlobalBox, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("BoundingVolumeFillGrid"), STAT_BoundingVolumeFillGrid, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("BoundingVolumeRemoveElement"), STAT_BoundingVolumeRemoveElement, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("BoundingVolumeUpdateElement"), STAT_BoundingVolumeUpdateElement, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("BoundingVolumeAddElement"), STAT_BoundingVolumeAddElement, STATGROUP_Chaos);

namespace Chaos
{
template<typename TPayloadType, typename T, int d>
struct TBVCellElement
{
	TAABB<T, d> Bounds;
	TPayloadType Payload;
	TVector<int32, 3> StartIdx;
	TVector<int32, 3> EndIdx;

	void Serialize(FChaosArchive& Ar)
	{
		TBox<T, d>::SerializeAsAABB(Ar, Bounds);
		Ar << Payload;
		Ar << StartIdx;
		Ar << EndIdx;
	}
};

template<typename TPayloadType, class T, int d>
FChaosArchive& operator<<(FChaosArchive& Ar, TBVCellElement<TPayloadType, T, d>& Elem)
{
	Elem.Serialize(Ar);
	return Ar;
}

template <typename T, int d>
struct TBVPayloadInfo
{
	int32 GlobalPayloadIdx;
	int32 DirtyPayloadIdx;
	TVector<int32, d> StartIdx;
	TVector<int32, d> EndIdx;

	void Serialize(FArchive& Ar)
	{
		Ar << GlobalPayloadIdx;
		Ar << DirtyPayloadIdx;
		Ar << StartIdx;
		Ar << EndIdx;
	}
};

template <typename T, int d>
FArchive& operator<<(FArchive& Ar, TBVPayloadInfo<T, d>& Info)
{
	Info.Serialize(Ar);
	return Ar;
}

template<typename InPayloadType, typename T = FReal, int d = 3>
class TBoundingVolume final : public ISpatialAcceleration<InPayloadType, T,d>
{
  public:
	using TPayloadType = InPayloadType;
	using PayloadType = TPayloadType;
	using TType = T;
	static constexpr int D = d;

	static constexpr int32 DefaultMaxCells = 15;
	static constexpr T DefaultMaxPayloadBounds = 100000;
	static constexpr ESpatialAcceleration StaticType = ESpatialAcceleration::BoundingVolume;
	TBoundingVolume()
		: ISpatialAcceleration<InPayloadType, T, d>(StaticType)
		, MaxPayloadBounds(DefaultMaxPayloadBounds)
	{
	}

	template <typename ParticleView>
	TBoundingVolume(const ParticleView& Particles, const bool bUseVelocity = false, const T Dt = 0, const int32 MaxCells = DefaultMaxCells, const T InMaxPayloadBounds = DefaultMaxPayloadBounds)
		: ISpatialAcceleration<InPayloadType, T, d>(StaticType)
		, MaxPayloadBounds(InMaxPayloadBounds)
	{
		Reinitialize(Particles, bUseVelocity, Dt, MaxCells);
	}

	TBoundingVolume(TBoundingVolume<TPayloadType, T, d>&& Other)
		: ISpatialAcceleration<InPayloadType, T, d>(StaticType)
		, MGlobalPayloads(MoveTemp(Other.MGlobalPayloads))
		, MGrid(MoveTemp(Other.MGrid))
		, MElements(MoveTemp(Other.MElements))
		, MDirtyElements(MoveTemp(Other.MDirtyElements))
		, MPayloadInfo(MoveTemp(Other.MPayloadInfo))
		, MaxPayloadBounds(Other.MaxPayloadBounds)
		, bIsEmpty(Other.bIsEmpty)
	{
	}

	//needed for tree of grids, should we have a more explicit way to copy an array of BVs to avoid this being public?
	TBoundingVolume(const TBoundingVolume<TPayloadType, T, d>& Other)
		: ISpatialAcceleration<InPayloadType, T, d>(StaticType)
		, MGlobalPayloads(Other.MGlobalPayloads)
		, MGrid(Other.MGrid)
		, MElements(Other.MElements.Copy())
		, MDirtyElements(Other.MDirtyElements)
		, MPayloadInfo(Other.MPayloadInfo)
		, MaxPayloadBounds(Other.MaxPayloadBounds)
		, bIsEmpty(Other.bIsEmpty)
	{
	}

public:

	virtual void DeepAssign(const ISpatialAcceleration<TPayloadType, T, d>& Other) override
	{

		check(Other.GetType() == ESpatialAcceleration::BoundingVolume);
		*this = static_cast<const TBoundingVolume<TPayloadType, T, d>&>(Other);
	}

	TBoundingVolume<TPayloadType, T, d>& operator=(const TBoundingVolume<TPayloadType, T, d>& Other)
	{
		ISpatialAcceleration<TPayloadType, FReal, 3>::DeepAssign(Other);
		MGlobalPayloads = Other.MGlobalPayloads;
		MGrid = Other.MGrid;
		MElements = Other.MElements;
		MDirtyElements = Other.MDirtyElements;
		MPayloadInfo = Other.MPayloadInfo;
		MaxPayloadBounds = Other.MaxPayloadBounds;
		bIsEmpty = Other.bIsEmpty;
		return *this;
	}

	TBoundingVolume<TPayloadType, T, d>& operator=(TBoundingVolume<TPayloadType, T, d>&& Other)
	{
		MGlobalPayloads = MoveTemp(Other.MGlobalPayloads);
		MGrid = MoveTemp(Other.MGrid);
		MElements = MoveTemp(Other.MElements);
		MDirtyElements = MoveTemp(Other.MDirtyElements);
		MPayloadInfo = MoveTemp(Other.MPayloadInfo);
		MaxPayloadBounds = Other.MaxPayloadBounds;
		bIsEmpty = Other.bIsEmpty;
		return *this;
	}

	virtual TUniquePtr<ISpatialAcceleration<TPayloadType, T, d>> Copy() const override
	{
		return TUniquePtr<ISpatialAcceleration<TPayloadType, T, d>>(new TBoundingVolume<TPayloadType, T, d>(*this));
	}

	template <typename ParticleView>
	void Reinitialize(const ParticleView& Particles, const bool bUseVelocity = false, const T Dt = 0, const int32 MaxCells = DefaultMaxCells)
	{
		GenerateTree(Particles, bUseVelocity, Dt, MaxCells);
	}

	TArray<TPayloadType> FindAllIntersectionsImp(const TAABB<T,d>& Intersection) const
	{
		struct FSimpleVisitor
		{
			FSimpleVisitor(TArray<TPayloadType>& InResults) : CollectedResults(InResults) {}
			bool VisitOverlap(const TSpatialVisitorData<TPayloadType>& Instance)
			{
				CollectedResults.Add(Instance.Payload);
				return true;
			}
			bool PrePreFilter(const TPayloadType&) const { return false; }
			const void* GetQueryPayload() const { return nullptr; }
			TArray<TPayloadType>& CollectedResults;
		};

		TArray<TPayloadType> Results;
		FSimpleVisitor Collector(Results);
		Overlap(Intersection, Collector);

		return Results;
	}

	virtual void Reset() override
	{
		MGlobalPayloads.Reset();
		MGrid.Reset();
		MElements.Reset();
		MDirtyElements.Reset();
		MPayloadInfo.Reset();
		bIsEmpty = true;
	}

	virtual bool RemoveElement(const TPayloadType& Payload) override
	{
		SCOPE_CYCLE_COUNTER(STAT_BoundingVolumeRemoveElement);
		if (const FPayloadInfo* PayloadInfo = MPayloadInfo.Find(Payload))
		{
			if (PayloadInfo->GlobalPayloadIdx != INDEX_NONE)
			{
				RemoveGlobalElement(Payload, *PayloadInfo);
			}
			else
			{
				RemoveElementFromExistingGrid(Payload, *PayloadInfo);
			}

			MPayloadInfo.Remove(Payload);
			return true;
		}
		return false;
	}

	virtual bool UpdateElement(const TPayloadType& Payload, const TAABB<T,d>& NewBounds, bool bHasBounds) override
	{
		SCOPE_CYCLE_COUNTER(STAT_BoundingVolumeUpdateElement);
		bool bElementExisted = true;
		if (FPayloadInfo* PayloadInfo = MPayloadInfo.Find(Payload))
		{
			ensure(bHasBounds || PayloadInfo->GlobalPayloadIdx != INDEX_NONE);
			if (PayloadInfo->GlobalPayloadIdx == INDEX_NONE)
			{
				RemoveElementFromExistingGrid(Payload, *PayloadInfo);
				AddElementToExistingGrid(Payload, *PayloadInfo, NewBounds, bHasBounds);
			}
			else if (bHasBounds)
			{
				RemoveGlobalElement(Payload, *PayloadInfo);
				AddElementToExistingGrid(Payload, *PayloadInfo, NewBounds, bHasBounds);
			}
		}
		else
		{
			bElementExisted = false;
			FPayloadInfo& NewPayloadInfo = MPayloadInfo.Add(Payload);
			AddElementToExistingGrid(Payload, NewPayloadInfo, NewBounds, bHasBounds);
		}
		return bElementExisted;
	}

	void UpdateElementWithoutDirty(const TPayloadType& Payload, const TAABB<T, 3>& NewBounds)
	{
		// Not implemented
		check(false);
	}

	inline void AddElement(const TPayloadBoundsElement<TPayloadType, T>& Payload)
	{
		// Not implemented 
		check(false);
	}

	inline void RecomputeBounds(bool bDynamicTree)
	{
	}

	inline int32 GetElementCount()
	{
		// Not implemented 
		check(false);
		return 0;
	}

	
	// Begin ISpatialAcceleration interface
	virtual TArray<TPayloadType> FindAllIntersections(const FAABB3& Box) const override { return FindAllIntersectionsImp(Box); }

	const TArray<TPayloadBoundsElement<TPayloadType, T>>& GlobalObjects() const
	{
		return MGlobalPayloads;
	}

	virtual void Raycast(const TVector<T, d>& Start, const TVector<T, d>& Dir, const T Length, ISpatialVisitor<TPayloadType, T>& Visitor) const override
	{
		TSpatialVisitor<TPayloadType, T> ProxyVisitor(Visitor);
		return Raycast(Start, Dir, Length, ProxyVisitor);
	}

	template <typename SQVisitor>
	bool RaycastFast(const TVector<T,d>& Start, FQueryFastData& CurData, SQVisitor& Visitor, const FVec3& Dir, const FVec3 InvDir, const bool bParallel[3]) const
	{
		return RaycastImp(Start, CurData, Visitor, Dir, InvDir, bParallel);
	}

	template <typename SQVisitor>
	bool RaycastFastSimd(const VectorRegister4Double& Start, FQueryFastData& CurData, SQVisitor& Visitor, const VectorRegister4Double& Dir, const VectorRegister4Double& InvDir, const VectorRegister4Double& Parallel, const VectorRegister4Double& Length) const
	{
		FVec3 StartReal;
		VectorStoreDouble3(Start, &StartReal[0]);
		return RaycastImp(StartReal, CurData, Visitor, CurData.Dir, CurData.InvDir, CurData.bParallel);
	}

	template <typename SQVisitor, bool bPruneDuplicates = true>
	void Raycast(const TVector<T, d>& Start, const TVector<T, d>& Dir, const T Length, SQVisitor& Visitor) const
	{
		FQueryFastData CurData(Dir, Length);
		RaycastImp<SQVisitor, bPruneDuplicates>(Start, CurData, Visitor, CurData.Dir, CurData.InvDir, CurData.bParallel);
	}

	void Sweep(const TVector<T, d>& Start, const TVector<T, d>& Dir, const T Length, const TVector<T, d> QueryHalfExtents, ISpatialVisitor<TPayloadType, T>& Visitor) const override
	{
		TSpatialVisitor<TPayloadType, T> ProxyVisitor(Visitor);
		Sweep(Start, Dir, Length, QueryHalfExtents, ProxyVisitor);
	}

	template <typename SQVisitor>
	bool SweepFast(const TVector<T,d>& Start, FQueryFastData& CurData, const TVector<T,d>& QueryHalfExtents, SQVisitor& Visitor, const FVec3& Dir, const FVec3 InvDir, const bool bParallel[3]) const
	{
		return SweepImp(Start, CurData, QueryHalfExtents, Visitor, Dir, InvDir, bParallel);
	}

	template <typename SQVisitor, bool bPruneDuplicates = true>
	void Sweep(const TVector<T, d>& Start, const TVector<T, d>& Dir, const T Length, const TVector<T, d> QueryHalfExtents, SQVisitor& Visitor) const
	{
		FQueryFastData CurData(Dir, Length);
		SweepImp<SQVisitor, bPruneDuplicates>(Start, CurData, QueryHalfExtents, Visitor, CurData.Dir, CurData.InvDir, CurData.bParallel);
	}

	virtual void Overlap(const TAABB<T, d>& QueryBounds, ISpatialVisitor<TPayloadType, T>& Visitor) const override
	{
		TSpatialVisitor<TPayloadType, T> ProxyVisitor(Visitor);
		Overlap(QueryBounds, ProxyVisitor);
	}

	template <typename SQVisitor>
	bool OverlapFast(const TAABB<T, d>& QueryBounds, SQVisitor& Visitor) const
	{
		return OverlapImp(QueryBounds, Visitor);
	}

	template <typename SQVisitor, bool bPruneDuplicates = true>
	void Overlap(const TAABB<T, d>& QueryBounds, SQVisitor& Visitor) const
	{
		OverlapImp<SQVisitor, bPruneDuplicates>(QueryBounds, Visitor);
	}
	
	/** Check if the leaf is dirty (if one of the payload have been updated)
	* @return Dirty boolean that indicates if the leaf is dirty or not
	*/
	bool IsLeafDirty() const
	{
		return false;
	}
	
	/** Set the dirty flag onto the leaf 
	* @param  bDirtyState Dirty flag to set 
	*/
	void SetDirtyState(const bool bDirtyState)
	{}
	
	virtual void Serialize(FChaosArchive& Ar) override
	{
		Ar.UsingCustomVersion(FExternalPhysicsCustomObjectVersion::GUID);
		if (Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) < FExternalPhysicsCustomObjectVersion::GlobalElementsHaveBounds)
		{
			TArray<TPayloadType> TmpPayloads;
			Ar << TmpPayloads;
			MGlobalPayloads.Reserve(TmpPayloads.Num());
			for (auto& Payload : TmpPayloads)
			{
				MGlobalPayloads.Add({ Payload, TAABB<T,d>(TVector<T,d>(TNumericLimits<T>::Lowest()), TVector<T,d>(TNumericLimits<T>::Max())) });
			}
			MaxPayloadBounds = DefaultMaxPayloadBounds;
		}
		else
		{
			Ar << MGlobalPayloads;
			Ar << MaxPayloadBounds;
		}

		Ar << MGrid;
		Ar << MElements;
		Ar << MDirtyElements;
		Ar << bIsEmpty;

		Ar << MPayloadInfo;
		

	}

	void GatherElements(TArray<TPayloadBoundsElement<TPayloadType, T>>& OutElements) const
	{
		OutElements.Reserve(GetReserveCount());
		OutElements.Append(MGlobalPayloads);
		
		for (const FCellElement& Elem : MDirtyElements)
		{
			OutElements.Add(TPayloadBoundsElement<TPayloadType,T>{Elem.Payload,Elem.Bounds});
		}

		const auto& Counts = MGrid.Counts();
		for (int32 X = 0; X < Counts[0]; ++X)
		{
			for (int32 Y = 0; Y < Counts[1]; ++Y)
			{
				for (int32 Z = 0; Z < Counts[2]; ++Z)
				{
					const auto& Elems = MElements(X, Y, Z);
					for (const auto& Elem : Elems)
					{
						//elements can be in multiple cells, only add for the first cell
						if (Elem.StartIdx == TVector<int32, 3>(X, Y, Z))
						{
							OutElements.Add(TPayloadBoundsElement<TPayloadType, T>{Elem.Payload, Elem.Bounds});
						}
					}
				}
			}
		}
	}

	int32 GetReserveCount() const
	{
		// Optimize for fewer memory allocations.
		const TVector<int32, d>& Counts = MGrid.Counts();
		const int32 GridCount = Counts[0] * Counts[1] * Counts[2] * MElements.Num();
		return MGlobalPayloads.Num() + MDirtyElements.Num() + GridCount;
	}

	TAABB<T, d> GetBounds() const
	{
		return TAABB<T, d>(MGrid.MinCorner(), MGrid.MaxCorner());
	}

private:

	using FCellElement = TBVCellElement<TPayloadType, T, d>;
	using FPayloadInfo = TBVPayloadInfo<T, d>;

	void RemoveGlobalElement(const TPayloadType& Payload, const FPayloadInfo& PayloadInfo)
	{
		ensure(PayloadInfo.DirtyPayloadIdx == INDEX_NONE);
		auto LastGlobalPayload = MGlobalPayloads.Last().Payload;
		if (!(LastGlobalPayload == Payload))
		{
			MPayloadInfo.FindChecked(LastGlobalPayload).GlobalPayloadIdx = PayloadInfo.GlobalPayloadIdx;
		}
		MGlobalPayloads.RemoveAtSwap(PayloadInfo.GlobalPayloadIdx);
	}

	template <typename SQVisitor, bool bPruneDuplicates = true>
	bool RaycastImp(const TVector<T, d>& Start, FQueryFastData& CurData, SQVisitor& Visitor, const FVec3& Dir, const FVec3 InvDir, const bool bParallel[3]) const
	{
		T TOI, ExitTime;

		for (const auto& Elem : MGlobalPayloads)
		{
			if (PrePreFilterHelper(Elem.Payload, Visitor))
			{
				continue;
			}

			const auto& InstanceBounds = Elem.Bounds;
			if (InstanceBounds.RaycastFast(Start,
				Dir, InvDir, bParallel, CurData.CurrentLength, CurData.InvCurrentLength, TOI, ExitTime))
			{
				TSpatialVisitorData<TPayloadType> VisitData(Elem.Payload, true, InstanceBounds);
				const bool bContinue = Visitor.VisitRaycast(VisitData, CurData);

				if (!bContinue)
				{
					return false;
				}
			}
		}

		for (const auto& Elem : MDirtyElements)
		{
			if (PrePreFilterHelper(Elem.Payload, Visitor))
			{
				continue;
			}

			const auto& InstanceBounds = Elem.Bounds;
			if (InstanceBounds.RaycastFast(Start, Dir,
				InvDir, bParallel, CurData.CurrentLength, CurData.InvCurrentLength, TOI, ExitTime))
			{
				TSpatialVisitorData<TPayloadType> VisitData(Elem.Payload, true, InstanceBounds);
				const bool bContinue = Visitor.VisitRaycast(VisitData, CurData);
				if (!bContinue)
				{
					return false;
				}
			}
		}

		if (bIsEmpty)
		{
			return true;
		}

		TAABB<T, d> GlobalBounds(MGrid.MinCorner(), MGrid.MaxCorner());
		TVector<T, d> NextStart;
		TVector<int32, d> CellIdx;
		bool bCellsLeft = MElements.Num() && GlobalBounds.RaycastFast(Start, Dir,
			InvDir, bParallel, CurData.CurrentLength, CurData.InvCurrentLength, TOI, NextStart);
		if (bCellsLeft)
		{
			CellIdx = MGrid.Cell(NextStart);
			FGridSet CellsVisited(MGrid.Counts());

			do
			{
				//gather all instances in current cell whose bounds intersect with ray
				const auto& Elems = MElements(CellIdx);
				//should we let callback know about max potential?

				for (const auto& Elem : Elems)
				{
					if (PrePreFilterHelper(Elem.Payload, Visitor))
					{
						continue;
					}

					if (bPruneDuplicates)
					{
						bool bSkip = false;
						if (Elem.StartIdx[0] != Elem.EndIdx[0] || Elem.StartIdx[1] != Elem.EndIdx[1] || Elem.StartIdx[2] != Elem.EndIdx[2])
						{
							for (int32 X = Elem.StartIdx[0]; X <= Elem.EndIdx[0]; ++X)
							{
								for (int32 Y = Elem.StartIdx[1]; Y <= Elem.EndIdx[1]; ++Y)
								{
									for (int32 Z = Elem.StartIdx[2]; Z <= Elem.EndIdx[2]; ++Z)
									{
										if (CellsVisited.Contains(TVector<int32, 3>(X, Y, Z)))
										{
											bSkip = true;
											break;
										}
									}
								}
							}

							if (bSkip)
							{
								continue;
							}
						}
					}
					const auto& InstanceBounds = Elem.Bounds;
					if (InstanceBounds.RaycastFast(Start,
							Dir, InvDir, bParallel, CurData.CurrentLength, CurData.InvCurrentLength, TOI, ExitTime))
					{
						TSpatialVisitorData<TPayloadType> VisitData(Elem.Payload, true, InstanceBounds);
						const bool bContinue = Visitor.VisitRaycast(VisitData, CurData);
						if (!bContinue)
						{
							return false;
						}
					}
				}

				// Early exit if the raycast has a hit and it has set as bloking hit, it is not necessary to navigate along the raycast
				if (Visitor.HasBlockingHit())
				{
					return false;
				}

				CellsVisited.Add(CellIdx);


				//find next cell

				//We want to know which plane we used to cross into next cell
				const TVector<T, d> CellCenter = MGrid.Location(CellIdx);
				const TVector<T, d>& Dx = MGrid.Dx();

				T Times[3];
				T BestTime = CurData.CurrentLength;
				bool bTerminate = true;
				for (int Axis = 0; Axis < d; ++Axis)
				{
					if (!bParallel[Axis])
					{
						const T CrossPoint = Dir[Axis] > 0 ? CellCenter[Axis] + Dx[Axis] / 2 : CellCenter[Axis] - Dx[Axis] / 2;
						const T Distance = CrossPoint - NextStart[Axis];	//note: CellCenter already has /2, we probably want to use the corner instead
						const T Time = Distance * InvDir[Axis];
						Times[Axis] = Time;
						if (Time < BestTime)
						{
							bTerminate = false;	//found at least one plane to pass through
							BestTime = Time;
						}
					}
					else
					{
						Times[Axis] = TNumericLimits<T>::Max();
					}
				}

				if (bTerminate)
				{
					return true;
				}

				for (int Axis = 0; Axis < d; ++Axis)
				{
					constexpr T Epsilon = 1e-2f;	//if raycast is slightly off we still count it as hitting the cell surface
					CellIdx[Axis] += (Times[Axis] <= BestTime + Epsilon) ? (Dir[Axis] > 0 ? 1 : -1) : 0;
					if (CellIdx[Axis] < 0 || CellIdx[Axis] >= MGrid.Counts()[Axis])
					{
						return true;
					}
				}

				NextStart = NextStart + Dir * BestTime;
			} while (true);
		}

		return true;
	}

	template <typename SQVisitor, bool bPruneDuplicates = true>
	bool SweepImp(const TVector<T, d>& Start, FQueryFastData& CurData, const TVector<T, d> QueryHalfExtents, SQVisitor& Visitor, const FVec3& Dir, const FVec3 InvDir, const bool bParallel[3]) const
	{
		T TOI = 0;
		for (const auto& Elem : MGlobalPayloads)
		{
			if (PrePreFilterHelper(Elem.Payload, Visitor))
			{
				continue;
			}

			const TAABB<T, d>& InstanceBounds = Elem.Bounds;
			const TVector<T, d> Min = InstanceBounds.Min() - QueryHalfExtents;
			const TVector<T, d> Max = InstanceBounds.Max() + QueryHalfExtents;
			TVector<T, d> TmpPosition;
			if (TAABB<T, d>(Min,Max).RaycastFast(Start, Dir, InvDir, bParallel, CurData.CurrentLength, CurData.InvCurrentLength, TOI, TmpPosition))
			{
				TSpatialVisitorData<TPayloadType> VisitData(Elem.Payload, true, InstanceBounds);
				const bool bContinue = Visitor.VisitSweep(VisitData, CurData);
				if (!bContinue)
				{
					return false;
				}
			}
		}

		for (const auto& Elem : MDirtyElements)
		{
			if (PrePreFilterHelper(Elem.Payload, Visitor))
			{
				continue;
			}

			const TAABB<T, d>& InstanceBounds = Elem.Bounds;
			const TVector<T, d> Min = InstanceBounds.Min() - QueryHalfExtents;
			const TVector<T, d> Max = InstanceBounds.Max() + QueryHalfExtents;
			TVector<T, d> TmpPosition;
			if (TAABB<T, d>(Min, Max).RaycastFast(Start, Dir, InvDir, bParallel, CurData.CurrentLength, CurData.InvCurrentLength, TOI, TmpPosition))
			{
				TSpatialVisitorData<TPayloadType> VisitData(Elem.Payload, true, InstanceBounds);
				const bool bContinue = Visitor.VisitSweep(VisitData, CurData);
				if (!bContinue)
				{
					return false;
				}
			}
		}

		if (bIsEmpty)
		{
			return true;
		}

		TAABB<T, d> GlobalBounds(MGrid.MinCorner() - QueryHalfExtents, MGrid.MaxCorner() + QueryHalfExtents);


		struct FCellIntersection
		{
			TVector<int32, d> CellIdx;
			T TOI;
		};

		const TVector<int32, d> StartMinIndex = MGrid.Cell(Start - QueryHalfExtents);
		const TVector<int32, d> StartMaxIndex = MGrid.Cell(Start + QueryHalfExtents);

		if (StartMinIndex == StartMaxIndex)
		{
			const TVector<T, d> End = Start + CurData.CurrentLength * Dir;
			const TVector<int32, d> EndMinIndex = MGrid.Cell(End  - QueryHalfExtents);
			const TVector<int32, d> EndMaxIndex = MGrid.Cell(End + QueryHalfExtents);
			if (StartMinIndex == EndMinIndex && StartMinIndex == EndMaxIndex)
			{
				//sweep is fully contained within one cell, this is a common special case - just test all elements
				const auto& Elems = MElements(StartMinIndex);
				TVector<T, d> TmpPoint;
				for (const auto& Elem : Elems)
				{
					const TAABB<T, d>& InstanceBounds = Elem.Bounds;
					const TVector<T, d> Min = InstanceBounds.Min() - QueryHalfExtents;
					const TVector<T, d> Max = InstanceBounds.Max() + QueryHalfExtents;
					if (TAABB<T, d>(Min, Max).RaycastFast(Start, Dir, InvDir, bParallel, CurData.CurrentLength, CurData.InvCurrentLength, TOI, TmpPoint))
					{
						TSpatialVisitorData<TPayloadType> VisitData(Elem.Payload, true, InstanceBounds);
						const bool bContinue = Visitor.VisitSweep(VisitData, CurData);
						if (!bContinue)
						{
							return false;
						}
					}
				}

				return true;
			}
		}

		FGridSet IdxsSeen(MGrid.Counts());
		FGridSet CellsVisited(MGrid.Counts());
		TArray<FCellIntersection> IdxsQueue;	//cells we need to visit

		for (int32 X = StartMinIndex[0]; X <= StartMaxIndex[0]; ++X)
		{
			for (int32 Y = StartMinIndex[1]; Y <= StartMaxIndex[1]; ++Y)
			{
				for (int32 Z = StartMinIndex[2]; Z <= StartMaxIndex[2]; ++Z)
				{
					const TVector<int32, 3> Idx(X, Y, Z);
					IdxsQueue.Add({Idx, 0 });
					IdxsSeen.Add(Idx);
				}
			}
		}

		TVector<T, d> HitPoint;
		const bool bInitialHit = GlobalBounds.RaycastFast(Start, Dir, InvDir, bParallel, CurData.CurrentLength, CurData.InvCurrentLength, TOI, HitPoint);
		if (bInitialHit)	//NOTE: it's possible to have a non empty IdxsQueue and bInitialHit be false. This is because the IdxsQueue works off clamped cells which we can skip
		{
			//Flood fill from inflated cell so that we get all cells along the ray
			TVector<int32, d> HitCellIdx = MGrid.Cell(HitPoint);

			if (!IdxsSeen.Contains(HitCellIdx))
			{
				ensure(TOI > 0);	//Not an initial overlap so TOI must be positive
				IdxsQueue.Add({ HitCellIdx, TOI });
			}

			const TVector<T, d> HalfDx = MGrid.Dx() * (T)0.5;

			int32 QueueIdx = 0;	//FIFO because early cells are more likely to block later cells we can skip
			while (QueueIdx < IdxsQueue.Num())
			{
				const FCellIntersection CellIntersection = IdxsQueue[QueueIdx++];
				if (CellIntersection.TOI > CurData.CurrentLength)
				{
					continue;
				}

				//ray still visiting this cell so check all neighbors
				check(d == 3);
				static const TVector<int32, 3> Neighbors[] =
				{
					//grid on z=-1 plane
					{-1, -1, -1}, {0, -1, -1}, {1, -1, -1},
					{-1, 0, -1}, {0, 0, -1}, {1, 0, -1},
					{-1, 1, -1}, {0, 1, -1}, {1, 1, -1},

					//grid on z=0 plane
					{-1, -1, 0}, {0, -1, 0}, {1, -1, 0},
					{-1, 0, 0},			 {1, 0, 0},
					{-1, 1, 0}, {0, 1, 0}, {1, 0, 0},

					//grid on z=1 plane
					{-1, -1, 1}, {0, -1, 1}, {1, -1, 1},
					{-1, 0, 1}, {0, 0, 1}, {1, 0, 1},
					{-1, 1, 1}, {0, 1, 1}, {1, 1, 1}
				};

				for (const TVector<int32, 3>& Neighbor : Neighbors)
				{
					const TVector<int32, 3> NeighborIdx = Neighbor + CellIntersection.CellIdx;
					bool bSkip = false;
					for (int32 Axis = 0; Axis < d; ++Axis)
					{
						if (NeighborIdx[Axis] < 0 || NeighborIdx[Axis] >= MGrid.Counts()[Axis])
						{
							bSkip = true;
							break;
						}
					}
					if (!bSkip && !IdxsSeen.Contains(NeighborIdx))
					{
						IdxsSeen.Add(NeighborIdx);

						const TVector<T, d> NeighborCenter = MGrid.Location(NeighborIdx);
						const TVector<T, d> Min = NeighborCenter - QueryHalfExtents - HalfDx;
						const TVector<T, d> Max = NeighborCenter + QueryHalfExtents + HalfDx;
						if (TAABB<T, d>(Min, Max).RaycastFast(Start, Dir, InvDir, bParallel, CurData.CurrentLength, CurData.InvCurrentLength, TOI, HitPoint))
						{
							IdxsQueue.Add({ NeighborIdx, TOI });	//should we sort by TOI?
						}
					}
				}

				//check if any instances in the cell are hit
				const auto& Elems = MElements(CellIntersection.CellIdx);
				for (const auto& Elem : Elems)
				{
					if (PrePreFilterHelper(Elem.Payload, Visitor))
					{
						continue;
					}

					if (bPruneDuplicates)
					{
						bool bSkip = false;
						if (Elem.StartIdx[0] != Elem.EndIdx[0] || Elem.StartIdx[1] != Elem.EndIdx[1] || Elem.StartIdx[2] != Elem.EndIdx[2])
						{
							for (int32 X = Elem.StartIdx[0]; X <= Elem.EndIdx[0]; ++X)
							{
								for (int32 Y = Elem.StartIdx[1]; Y <= Elem.EndIdx[1]; ++Y)
								{
									for (int32 Z = Elem.StartIdx[2]; Z <= Elem.EndIdx[2]; ++Z)
									{
										if (CellsVisited.Contains(TVector<int32, 3>(X, Y, Z)))
										{
											bSkip = true;
											break;
										}
									}
								}
							}

							if (bSkip)
							{
								continue;
							}
						}
					}

					const TAABB<T, d>& InstanceBounds = Elem.Bounds;
					const TVector<T, d> Min = InstanceBounds.Min() - QueryHalfExtents;
					const TVector<T, d> Max = InstanceBounds.Max() + QueryHalfExtents;
					if (TAABB<T, d>(Min,Max).RaycastFast(Start,
							Dir, InvDir, bParallel, CurData.CurrentLength, CurData.InvCurrentLength, TOI, HitPoint))
					{
						TSpatialVisitorData<TPayloadType> VisitData(Elem.Payload, true, InstanceBounds);
						const bool bContinue = Visitor.VisitSweep(VisitData, CurData);
						if (!bContinue)
						{
							return false;
						}
					}
				}
				CellsVisited.Add(CellIntersection.CellIdx);
			}
		}

		return true;
	}

	template <typename SQVisitor, bool bPruneDuplicates = true>
	bool OverlapImp(const TAABB<T, d>& QueryBounds, SQVisitor& Visitor) const
	{
		for (const auto& Elem : MGlobalPayloads)
		{
			if (PrePreFilterHelper(Elem.Payload, Visitor))
			{
				continue;
			}

			const TAABB<T, d>& InstanceBounds = Elem.Bounds;
			if (QueryBounds.Intersects(InstanceBounds))
			{
				TSpatialVisitorData<TPayloadType> VisitData(Elem.Payload, true, InstanceBounds);
				if (Visitor.VisitOverlap(VisitData) == false)
				{
					return false;
				}
			}
		}

		for (const auto& Elem : MDirtyElements)
		{
			if (PrePreFilterHelper(Elem.Payload, Visitor))
			{
				continue;
			}

			const TAABB<T, d>& InstanceBounds = Elem.Bounds;
			if (QueryBounds.Intersects(InstanceBounds))
			{
				TSpatialVisitorData<TPayloadType> VisitData(Elem.Payload, true, InstanceBounds);
				if (Visitor.VisitOverlap(VisitData) == false)
				{
					return false;
				}
			}
		}

		if (bIsEmpty)
		{
			return true;
		}

		TAABB<T, d> GlobalBounds(MGrid.MinCorner(), MGrid.MaxCorner());

		const TVector<int32, d> StartIndex = MGrid.Cell(QueryBounds.Min());
		const TVector<int32, d> EndIndex = MGrid.Cell(QueryBounds.Max());
		TSet<FUniqueIdx> InstancesSeen;

		for (int32 X = StartIndex[0]; X <= EndIndex[0]; ++X)
		{
			for (int32 Y = StartIndex[1]; Y <= EndIndex[1]; ++Y)
			{
				for (int32 Z = StartIndex[2]; Z <= EndIndex[2]; ++Z)
				{
					const auto& Elems = MElements(X, Y, Z);
					for (const auto& Elem : Elems)
					{
						if (PrePreFilterHelper(Elem.Payload, Visitor))
						{
							continue;
						}
						if (bPruneDuplicates)
						{
							if (InstancesSeen.Contains(GetUniqueIdx(Elem.Payload)))
							{
								continue;
							}
							InstancesSeen.Add(GetUniqueIdx(Elem.Payload));
						}
						const TAABB<T, d>& InstanceBounds = Elem.Bounds;
						if (QueryBounds.Intersects(InstanceBounds))
						{
							TSpatialVisitorData<TPayloadType> VisitData(Elem.Payload, true, InstanceBounds);
							if (Visitor.VisitOverlap(VisitData) == false)
							{
								return false;
							}
						}
					}
				}
			}
		}

		return true;
	}

	template <typename ParticleView>
	void GenerateTree(const ParticleView& Particles, const bool bUseVelocity, const T Dt, const int32 MaxCells)
	{
		MGlobalPayloads.Reset();
		MPayloadInfo.Reset();
		bIsEmpty = true;

		if (!Particles.Num())
		{
			bIsEmpty = true;
			return;
		}

		SCOPE_CYCLE_COUNTER(STAT_BoundingVolumeGenerateTree);
		TArray<TAABB<T, d>> AllBounds;
		TArray<bool> HasBounds;

		AllBounds.SetNum(Particles.Num());
		HasBounds.SetNum(Particles.Num());
		T MaxPayloadBoundsCopy = MaxPayloadBounds;
		auto GetValidBounds = [MaxPayloadBoundsCopy, bUseVelocity, Dt](const auto& Particle, TAABB<T,d>& OutBounds) -> bool
		{
			if (HasBoundingBox(Particle))
			{
				OutBounds = ComputeWorldSpaceBoundingBox(Particle, bUseVelocity, Dt);	//todo: avoid computing on the fly
				//todo: check if bounds are outside of something we deem reasonable (i.e. object really far out in space)
				if (OutBounds.Extents().Max() < MaxPayloadBoundsCopy)
				{
					return true;
				}
			}

			return false;
		};

		//compute DX and fill global payloads
		auto& GlobalPayloads = MGlobalPayloads;
		auto& PayloadInfos = MPayloadInfo;
		T NumObjectsWithBounds = 0;

		auto ComputeBoxAndDx = [&Particles, &AllBounds, &HasBounds, &GlobalPayloads, &PayloadInfos, &GetValidBounds, &NumObjectsWithBounds](TAABB<T,d>& OutGlobalBox, bool bFirstPass) -> T
		{
			SCOPE_CYCLE_COUNTER(STAT_BoundingVolumeComputeGlobalBox);
			OutGlobalBox = TAABB<T, d>::EmptyAABB();
			constexpr T InvD = (T)1 / d;
			int32 Idx = 0;
			T Dx = 0;
			NumObjectsWithBounds = 0;
			for (auto& Particle : Particles)
			{
				TAABB<T,d>& Bounds = AllBounds[Idx];
				if ((bFirstPass && GetValidBounds(Particle, Bounds)) || (!bFirstPass && HasBounds[Idx]))
				{
					HasBounds[Idx] = true;
					OutGlobalBox.GrowToInclude(Bounds);
					Dx += TVector<T, d>::DotProduct(Bounds.Extents(), TVector<T, d>(1)) * InvD;;
					NumObjectsWithBounds += 1;
				}
				else if(bFirstPass)
				{
					HasBounds[Idx] = false;
					auto Payload = Particle.template GetPayload<TPayloadType>(Idx);

					const int32 GlobalPayloadIdx = GlobalPayloads.Num();
					bool bTooBig = HasBoundingBox(Particle);	//todo: avoid this as it was already called in GetValidBounds
					GlobalPayloads.Add({ Payload, bTooBig ? Bounds : TAABB<T,d>(TVector<T,d>(TNumericLimits<T>::Lowest()), TVector<T,d>(TNumericLimits<T>::Max())) });
					PayloadInfos.Add(Payload, FPayloadInfo{ GlobalPayloadIdx, INDEX_NONE });
				}
				++Idx;
			}
			Dx = NumObjectsWithBounds > 0 ? Dx / NumObjectsWithBounds : (T)0;
			return Dx;
		};

		TAABB<T, d> GlobalBox;
		T Dx = ComputeBoxAndDx(GlobalBox, /*bFirstPass=*/true);

		if (FBoundingVolumeCVars::FilterFarBodies)
		{
			bool bRecomputeBoxAndDx = false;
			int32 Idx = 0;
			for (auto& Particle : Particles)
			{
				if (HasBounds[Idx])
				{
					bool bEvictElement = false;
					const auto& WorldSpaceBox = AllBounds[Idx];
					const TVector<T, d> MinToDXRatio = WorldSpaceBox.Min() / Dx;
					for (int32 Axis = 0; Axis < d; ++Axis)
					{
						if (FMath::Abs(MinToDXRatio[Axis]) > 1e7)
						{
							bEvictElement = true;
							break;
						}
					}

					if (bEvictElement)
					{
						bRecomputeBoxAndDx = true;
						HasBounds[Idx] = false;
						auto Payload = Particle.template GetPayload<TPayloadType>(Idx);

						const int32 GlobalPayloadIdx = GlobalPayloads.Num();
						GlobalPayloads.Add({ Payload, WorldSpaceBox });
						MPayloadInfo.Add(Payload, FPayloadInfo{ GlobalPayloadIdx, INDEX_NONE});
					}
				}

				++Idx;
			}

			if (bRecomputeBoxAndDx)
			{
				Dx = ComputeBoxAndDx(GlobalBox, /*bFirstPass=*/false);
			}
		}

		TVector<int32, d> Cells = Dx > 0 ? GlobalBox.Extents() / Dx : TVector<int32, d>(MaxCells);
		Cells += TVector<int32, d>(1);
		for (int32 Axis = 0; Axis < d; ++Axis)
		{
			if (Cells[Axis] > MaxCells)
				Cells[Axis] = MaxCells;
			if (!(ensure(Cells[Axis] >= 0)))	//seeing this because GlobalBox is huge leading to int overflow. Need to investigate why bounds get so big
			{
				Cells[Axis] = MaxCells;
			}
		}

#if ENABLE_NAN_DIAGNOSTIC
		if (!ensure(!GlobalBox.Min().ContainsNaN() && !GlobalBox.Max().ContainsNaN()))
		{
			UE_LOGF(LogChaos, Error, "BoundingVolume computed invalid GlobalBox from bounds: GlobalBox.Min(): (%f, %f, %f), GlobalBox.Max(): (%f, %f, %f)",
				GlobalBox.Min().X, GlobalBox.Min().Y, GlobalBox.Min().Z, GlobalBox.Max().X, GlobalBox.Max().Y, GlobalBox.Max().Z);
		}
#endif

		MGrid = TUniformGrid<T, d>(GlobalBox.Min(), GlobalBox.Max(), Cells);
		MElements = TArrayND<TArray<FCellElement>, d>(MGrid);

		//insert into grid cells
		T NumObjectsInCells = 0;
		{
			SCOPE_CYCLE_COUNTER(STAT_BoundingVolumeFillGrid);
			int32 Idx = 0;
			for (auto& Particle : Particles)
			{
				if (HasBounds[Idx])
				{
					const TAABB<T, d>& ObjectBox = AllBounds[Idx];
					NumObjectsWithBounds += 1;
					const auto StartIndex = MGrid.Cell(ObjectBox.Min());
					const auto EndIndex = MGrid.Cell(ObjectBox.Max());
					
					auto Payload = Particle.template GetPayload<TPayloadType>(Idx);
					MPayloadInfo.Add(Payload, FPayloadInfo{ INDEX_NONE, INDEX_NONE, StartIndex, EndIndex });

					for (int32 x = StartIndex[0]; x <= EndIndex[0]; ++x)
					{
						for (int32 y = StartIndex[1]; y <= EndIndex[1]; ++y)
						{
							for (int32 z = StartIndex[2]; z <= EndIndex[2]; ++z)
							{
								MElements(x, y, z).Add({ ObjectBox, Payload, StartIndex, EndIndex });
								NumObjectsInCells += 1;
							}
						}
					}
				}
				++Idx;
			}
		}

		bIsEmpty = NumObjectsInCells == 0;
		UE_LOGF(LogChaos, Verbose, "Generated Tree with (%d, %d, %d) Nodes and %f Per Cell", MGrid.Counts()[0], MGrid.Counts()[1], MGrid.Counts()[2], NumObjectsInCells / NumObjectsWithBounds);
	}

	void RemoveElementFromExistingGrid(const TPayloadType& Payload, const FPayloadInfo& PayloadInfo)
	{
		ensure(PayloadInfo.GlobalPayloadIdx == INDEX_NONE);
		if (PayloadInfo.DirtyPayloadIdx == INDEX_NONE)
		{
			for (int32 X = PayloadInfo.StartIdx[0]; X <= PayloadInfo.EndIdx[0]; ++X)
			{
				for (int32 Y = PayloadInfo.StartIdx[1]; Y <= PayloadInfo.EndIdx[1]; ++Y)
				{
					for (int32 Z = PayloadInfo.StartIdx[2]; Z <= PayloadInfo.EndIdx[2]; ++Z)
					{
						TArray<FCellElement>& Elems = MElements(X, Y, Z);
						int32 ElemIdx = 0;
						for (FCellElement& Elem : Elems)
						{
							if (Elem.Payload == Payload)
							{
								Elems.RemoveAtSwap(ElemIdx);
								break;
							}
							++ElemIdx;
						}
					}
				}
			}
		}
		else
		{
			//TODO: should we skip this if dirty and still dirty?
			auto LastPayload = MDirtyElements.Last().Payload;
			if (!(LastPayload == Payload))
			{
				MPayloadInfo.FindChecked(LastPayload).DirtyPayloadIdx = PayloadInfo.DirtyPayloadIdx;
			}
			MDirtyElements.RemoveAtSwap(PayloadInfo.DirtyPayloadIdx);
		}
	}

	void AddElementToExistingGrid(const TPayloadType& Payload, FPayloadInfo& PayloadInfo, const TAABB<T, d>& NewBounds, bool bHasBounds)
	{
		bool bTooBig = false;
		if (bHasBounds)
		{
			if (NewBounds.Extents().Max() > MaxPayloadBounds)
			{
				bTooBig = true;
				bHasBounds = false;
			}
		}

		if(bHasBounds)
		{
			PayloadInfo.GlobalPayloadIdx = INDEX_NONE;

			if (bIsEmpty == false)
			{
				//add payload to appropriate cells
				TVector<int32, 3> StartIndex = MGrid.CellUnsafe(NewBounds.Min());
				TVector<int32, 3> EndIndex = MGrid.CellUnsafe(NewBounds.Max());

				bool bDirty = false;
				for (int Axis = 0; Axis < d; ++Axis)
				{
					if (StartIndex[Axis] < 0 || EndIndex[Axis] >= MGrid.Counts()[Axis])
					{
						bDirty = true;
						break;
					}
				}

				if (!bDirty)
				{
					PayloadInfo.DirtyPayloadIdx = INDEX_NONE;
					PayloadInfo.StartIdx = StartIndex;
					PayloadInfo.EndIdx = EndIndex;

					for (int32 x = StartIndex[0]; x <= EndIndex[0]; ++x)
					{
						for (int32 y = StartIndex[1]; y <= EndIndex[1]; ++y)
						{
							for (int32 z = StartIndex[2]; z <= EndIndex[2]; ++z)
							{
								MElements(x, y, z).Add({ NewBounds, Payload, StartIndex, EndIndex });
							}
						}
					}

					return;
				}
			}

			PayloadInfo.DirtyPayloadIdx = MDirtyElements.Num();
			MDirtyElements.Add({ NewBounds, Payload });
		}
		else
		{
			PayloadInfo.GlobalPayloadIdx = MGlobalPayloads.Num();
			PayloadInfo.DirtyPayloadIdx = INDEX_NONE;
			MGlobalPayloads.Add({ Payload, bTooBig ? NewBounds : TAABB<T,d>(TVector<T,d>(TNumericLimits<T>::Lowest()), TVector<T,d>(TNumericLimits<T>::Max())) });
		}
	}

	TArray<TPayloadType> FindAllIntersectionsHelper(const TAABB<T, d>& ObjectBox) const
	{
		TArray<TPayloadType> Intersections;
		const auto StartIndex = MGrid.Cell(ObjectBox.Min());
		const auto EndIndex = MGrid.Cell(ObjectBox.Max());
		for (int32 x = StartIndex[0]; x <= EndIndex[0]; ++x)
		{
			for (int32 y = StartIndex[1]; y <= EndIndex[1]; ++y)
			{
				for (int32 z = StartIndex[2]; z <= EndIndex[2]; ++z)
				{
					const TArray<FCellElement>& CellElements = MElements(x, y, z);
					Intersections.Reserve(Intersections.Num() + CellElements.Num());
					for (const FCellElement& Elem : CellElements)
					{
						if (ObjectBox.Intersects(Elem.Bounds))
						{
							Intersections.Add(Elem.Payload);
						}
					}
				}
			}
		}

		Algo::Sort(Intersections,[](const TPayloadType& A,const TPayloadType& B)
		{
			return GetUniqueIdx(A) < GetUniqueIdx(B);
		});

		for (int32 i = Intersections.Num() - 1; i > 0; i--)
		{
			if (Intersections[i] == Intersections[i - 1])
			{
				Intersections.RemoveAtSwap(i, EAllowShrinking::No);
			}
		}

		return Intersections;
	}

	/** Similar to a TSet but acts specifically on grids */
	struct FGridSet
	{
		FGridSet(TVector<int32, 3> Size)
			: NumX(Size[0])
			, NumY(Size[1])
			, NumZ(Size[2])
		{
			int32 BitsNeeded = NumX * NumY * NumZ;
			int32 BytesNeeded = 1 + (BitsNeeded) / 8;
			Data = new uint8[BytesNeeded];
			FMemory::Memzero(Data, BytesNeeded);
		}

		bool Contains(const TVector<int32, 3>& Coordinate)
		{
			//Most sweeps are straight down the Z so store as adjacent Z, then Y, then X
			int32 Idx = (NumY * NumZ) * Coordinate[0] + (NumZ * Coordinate[1]) + Coordinate[2];
			int32 ByteIdx = Idx / 8;
			int32 BitIdx = Idx % 8;
			bool bContains = (Data[ByteIdx] >> BitIdx) & 0x1;
			return bContains;
		}

		void Add(const TVector<int32, 3>& Coordinate)
		{
			//Most sweeps are straight down the Z so store as adjacent Z, then Y, then X
			int32 Idx = (NumY * NumZ) * Coordinate[0] + (NumZ * Coordinate[1]) + Coordinate[2];
			int32 ByteIdx = Idx / 8;
			int32 BitIdx = Idx % 8;
			uint8 Mask = static_cast<uint8>(1 << BitIdx);
			Data[ByteIdx] |= Mask;
		}

		~FGridSet()
		{
			delete[] Data;
		}

	private:
		int32 NumX;
		int32 NumY;
		int32 NumZ;
		uint8* Data;
	};

	TArray<TPayloadBoundsElement<TPayloadType, T>> MGlobalPayloads;
	TUniformGrid<T, d> MGrid;
	TArrayND<TArray<FCellElement>, d> MElements;
	TArray<FCellElement> MDirtyElements;
	TArrayAsMap<TPayloadType, FPayloadInfo> MPayloadInfo;
	T MaxPayloadBounds;
	bool bIsEmpty;

	friend ::FChaosVDDataWrapperUtils;
};

template<typename TPayloadType, class T, int d>
FArchive& operator<<(FArchive& Ar, TBoundingVolume<TPayloadType, T, d>& BoundingVolume)
{
	check(false);
	return Ar;
}

template<typename TPayloadType, class T, int d>
FArchive& operator<<(FChaosArchive& Ar, TBoundingVolume<TPayloadType, T, d>& BoundingVolume)
{
	BoundingVolume.Serialize(Ar);
	return Ar;
}

#if !UE_MERGED_MODULES
#if PLATFORM_COMPILER_CLANG
extern template class CHAOS_API Chaos::TBoundingVolume<int32, Chaos::FReal, 3>;
extern template class CHAOS_API Chaos::TBoundingVolume<Chaos::FAccelerationStructureHandle, Chaos::FReal, 3>;
#else
extern template class TBoundingVolume<int32, FReal, 3>;
extern template class TBoundingVolume<class FAccelerationStructureHandle, FReal, 3>;
#endif
#endif

}
