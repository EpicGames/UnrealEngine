// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Array.h"
#include "Chaos/AABB.h"
#include "Chaos/Defines.h"
#include "Chaos/GeometryParticles.h"
#include "Chaos/ParticleHandleFwd.h"
#include "Chaos/BoundingVolume.h"
#include "Chaos/Transform.h"
#include "ChaosLog.h"
#include "Chaos/ISpatialAcceleration.h"
#include "Templates/Models.h"

template <typename T, bool>
struct TBVHLeafTraits
{
};

template <typename T>
struct TBVHLeafTraits<T, true>
{
	using TPayloadType = typename T::TPayloadType;
};

template <typename T>
struct TBVHLeafTraits<T, false>
{
	using TPayloadType = typename T::ElementType;
};

struct CComplexBVHLeaf
{
	template <typename T>
	auto Requires(typename T::TPayloadType) ->void;
};

namespace Chaos
{

template <typename T, int d>
struct TBVHNode
{
	TBVHNode() {}
	~TBVHNode() = default;
	TVector<T, d> MMin, MMax;
	int32 MAxis;
	TArray<int32> MChildren;
	int32 LeafIndex;
};

template <typename T, int d>
FArchive& operator<<(FArchive& Ar, TBVHNode<T,d>& LeafNode)
{
	return Ar << LeafNode.LeafIndex << LeafNode.MAxis << LeafNode.MChildren << LeafNode.MMax << LeafNode.MMin;
}

template<class OBJECT_ARRAY, class LEAF_TYPE, class T = FReal, int d = 3>
class TBoundingVolumeHierarchy final : public ISpatialAcceleration<int32, T,d>
{
  public:
	static constexpr int32 DefaultMaxLevels = 12;
	static constexpr bool DefaultAllowMultipleSplitting = false;
	static constexpr bool DefaultUseVelocity = false;
	static constexpr T DefaultDt = 0;
	using TPayloadType = typename TBVHLeafTraits<LEAF_TYPE, TModels_V<CComplexBVHLeaf, LEAF_TYPE>>::TPayloadType;

	TBoundingVolumeHierarchy()
		: MObjects(nullptr)
		, MMaxLevels(DefaultMaxLevels)
	{
	}

	CHAOS_API TBoundingVolumeHierarchy(const OBJECT_ARRAY& Objects, const int32 MaxLevels = DefaultMaxLevels, const bool bUseVelocity = DefaultUseVelocity, const T Dt = DefaultDt);
	CHAOS_API TBoundingVolumeHierarchy(const OBJECT_ARRAY& Objects, const TArray<uint32>& ActiveIndices, const int32 MaxLevels = DefaultMaxLevels, const bool bUseVelocity = DefaultUseVelocity, const T Dt = DefaultDt);
	
	TBoundingVolumeHierarchy(const TBoundingVolumeHierarchy<OBJECT_ARRAY, LEAF_TYPE, T, d>& Other) = delete;
	TBoundingVolumeHierarchy(TBoundingVolumeHierarchy<OBJECT_ARRAY, LEAF_TYPE, T, d>&& Other)
	    : MObjects(Other.MObjects), MGlobalObjects(MoveTemp(Other.MGlobalObjects)), MWorldSpaceBoxes(MoveTemp(Other.MWorldSpaceBoxes)), MMaxLevels(Other.MMaxLevels), Elements(MoveTemp(Other.Elements)), Leafs(MoveTemp(Other.Leafs))
	{
	}

	virtual void DeepAssign(const ISpatialAcceleration<int32, T, d>& Other) override
	{

		check(Other.GetType() == ESpatialAcceleration::AABBTreeBV);
		*this = static_cast<const TBoundingVolumeHierarchy<OBJECT_ARRAY, LEAF_TYPE, T, d>&>(Other);
	}

	TBoundingVolumeHierarchy& operator=(const TBoundingVolumeHierarchy<OBJECT_ARRAY, LEAF_TYPE, T, d>& Other)
	{
		ISpatialAcceleration<int32, T, d>::DeepAssign(Other);
		MObjects = Other.MObjects;
		MGlobalObjects = Other.MGlobalObjects;
		MWorldSpaceBoxes = Other.MWorldSpaceBoxes;
		MMaxLevels = Other.MMaxLevels;
		Elements = Other.Elements;
		Leafs = Other.Leafs;
		return *this;
	}

	TBoundingVolumeHierarchy& operator=(TBoundingVolumeHierarchy<OBJECT_ARRAY, LEAF_TYPE, T, d>&& Other)
	{
		MObjects = Other.MObjects;
		MGlobalObjects = MoveTemp(Other.MGlobalObjects);
		MWorldSpaceBoxes = MoveTemp(Other.MWorldSpaceBoxes);
		MMaxLevels = Other.MMaxLevels;
		Elements = MoveTemp(Other.Elements);
		Leafs = MoveTemp(Other.Leafs);
		return *this;
	}

	CHAOS_API void UpdateHierarchy(const bool AllowMultipleSplitting = DefaultAllowMultipleSplitting, const bool bUseVelocity = DefaultUseVelocity, const T Dt = DefaultDt);
	CHAOS_API void UpdateHierarchy(const TArray<uint32>& ActiveIndices, const bool AllowMultipleSplitting = DefaultAllowMultipleSplitting, const bool bUseVelocity = DefaultUseVelocity, const T Dt = DefaultDt);

	void Reinitialize(bool bUseVelocity, T Dt)
	{
		UpdateHierarchy(DefaultAllowMultipleSplitting, bUseVelocity, Dt);
	}

	void Reinitialize(const TArray<uint32>& ActiveIndices, bool bUseVelocity, T Dt)
	{
		UpdateHierarchy(ActiveIndices, DefaultAllowMultipleSplitting, bUseVelocity, Dt);
	}

	template<class T_INTERSECTION>
	TArray<int32> FindAllIntersectionsImp(const T_INTERSECTION& Intersection) const
	{
		if (Elements.Num())
		{
			TArray<int32> IntersectionList = FindAllIntersectionsHelper(Elements[0], Intersection);
			IntersectionList.Append(MGlobalObjects);
			return IntersectionList;
		}
		else
		{
			return MGlobalObjects;
		}
	}

	// Calls the visitor for every overlapping leaf.
	// @tparam TVisitor void(const LEAF_TYPE& Leaf)
	template<typename TVisitor>
	void VisitAllIntersections(const FAABB3& LocalBounds, const TVisitor& Visitor) const
	{
		if (Elements.Num())
		{
			VisitAllIntersectionsRecursive(Elements[0], LocalBounds, Visitor);
		}
	}

	// Begin ISpatialAcceleration interface
	TArray<int32> FindAllIntersections(const FAABB3& Box) const { return FindAllIntersectionsImp(Box); }
	TArray<int32> FindAllIntersections(const TSpatialRay<T,d>& Ray) const { return FindAllIntersectionsImp(Ray); }
	TArray<int32> FindAllIntersections(const TVector<T, d>& Point) const { return FindAllIntersectionsImp(Point); }
	CHAOS_API TArray<int32> FindAllIntersections(const TGeometryParticles<T, d>& InParticles, const int32 i) const;
	// End ISpatialAcceleration interface

	const TArray<int32>& GlobalObjects() const
	{
		return MGlobalObjects;
	}

	// TODO(mlentine): Need to move this elsewhere; probably on CollisionConstraint
	CHAOS_API const TAABB<T, d>& GetWorldSpaceBoundingBox(const TGeometryParticles<T, d>& InParticles, const int32 Index);

#if !UE_BUILD_SHIPPING
	CHAOS_API virtual void DebugDraw(ISpacialDebugDrawInterface<T>* InInterface) const override;
#endif

	CHAOS_API void Serialize(FArchive& Ar);

	virtual void Serialize(FChaosArchive& Ar) override
	{
		check(false);
	}

  private:
	void PrintTree(FString Prefix, const TBVHNode<T,d>* MyNode) const
	{
		UE_LOGF(LogChaos, Verbose, "%lsNode has Box: (%f, %f, %f) to (%f, %f, %f) with %d Children", *Prefix, MyNode->MMin[0], MyNode->MMin[1], MyNode->MMin[2], MyNode->MMax[0], MyNode->MMax[1], MyNode->MMax[2], MyNode->MChildren.Num());
		for (auto& Child : MyNode->MChildren)
		{
			PrintTree(Prefix + " ", &Elements[Child]);
		}
	}

	template<typename TVisitor>
	void VisitAllIntersectionsRecursive(const TBVHNode<T, d>& MyNode, const FAABB3& ObjectBox, const TVisitor& Visitor) const
	{
		TAABB<T, d> Box(MyNode.MMin, MyNode.MMax);
		if (!Box.Intersects(ObjectBox))
		{
			return;
		}

		if (MyNode.MChildren.Num() == 0)
		{
			Visitor(Leafs[MyNode.LeafIndex]);
			return;
		}

		int32 NumChildren = MyNode.MChildren.Num();
		for (int32 Child = 0; Child < NumChildren; ++Child)
		{
			VisitAllIntersectionsRecursive(Elements[MyNode.MChildren[Child]], ObjectBox, Visitor);
		}
	}


	CHAOS_API TArray<int32> FindAllIntersectionsHelper(const TBVHNode<T,d>& MyNode, const TVector<T, d>& Point) const;
	CHAOS_API TArray<int32> FindAllIntersectionsHelper(const TBVHNode<T,d>& MyNode, const TAABB<T, d>& ObjectBox) const;
	CHAOS_API TArray<int32> FindAllIntersectionsHelper(const TBVHNode<T, d>& MyNode, const TSpatialRay<T, d>& Ray) const;

	template <typename QUERY_OBJECT>
	void FindAllIntersectionsHelperRecursive(const TBVHNode<T,d>& MyNode, const QUERY_OBJECT& ObjectBox, TArray<int32>& AccumulateElements) const;
	void UpdateHierarchyImp(const TArray<int32>& AllObjects, const bool bAllowMultipleSplitting, const bool bUseVelocity, const T Dt);

	int32 GenerateNextLevel(const TVector<T, d>& GlobalMin, const TVector<T, d>& GlobalMax, const TArray<int32>& Objects, const int32 Axis, const int32 Level, const bool AllowMultipleSplitting, TArray<TBVHNode<T, d>>& OutElements, TArray<LEAF_TYPE>& OutLeafs);
	int32 GenerateNextLevel(const TVector<T, d>& GlobalMin, const TVector<T, d>& GlobalMax, const TArray<int32>& Objects, const int32 Level, TArray<TBVHNode<T, d>>& OutElements, TArray<LEAF_TYPE>& OutLeafs);

	OBJECT_ARRAY const* MObjects;
	TArray<int32> MGlobalObjects;
	TMap<int32, TAABB<T, d>> MWorldSpaceBoxes;
	TArray<int32> MScratchAllObjects;
	int32 MMaxLevels;
	TArray<TBVHNode<T,d>> Elements;
	TArray<LEAF_TYPE> Leafs;
};

template<class OBJECT_ARRAY, class LEAF_TYPE, class T, int d>
FArchive& operator<<(FArchive& Ar, TBoundingVolumeHierarchy<OBJECT_ARRAY, LEAF_TYPE, T, d>& BVH)
{
	BVH.Serialize(Ar);
	return Ar;
}

}
