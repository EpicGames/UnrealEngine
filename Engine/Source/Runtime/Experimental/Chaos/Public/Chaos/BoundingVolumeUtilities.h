// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Array.h"
#include "Chaos/Box.h"
#include "Chaos/Defines.h"
#include "Chaos/Framework/Parallel.h"
#include "Chaos/GeometryParticles.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/PBDRigidParticles.h"
#include "Chaos/Particles.h"
#include "Chaos/Sphere.h"
#include "Chaos/Transform.h"

#define MIN_NUM_OBJECTS 5

namespace Chaos
{
extern float Chaos_Bounds_MaxInflationScale;

inline FVec3 ComputeBoundsThickness(FVec3 Vel, FReal Dt, FReal MinBoundsThickness, FReal MaxBoundsThickness, FReal BoundsVelocityInflation)
{
	for (int i = 0; i < 3; ++i)
	{
		const FReal BoundsThickness = FMath::Abs(Vel[i]) * Dt * BoundsVelocityInflation;
		Vel[i] = FMath::Clamp(BoundsThickness, MinBoundsThickness, MaxBoundsThickness);
	}
	return Vel;
}

inline FVec3 ComputeBoundsThickness(const TPBDRigidParticles<FReal, 3>& InParticles, FReal Dt, int32 BodyIndex, FReal MinBoundsThickness, FReal BoundsVelocityInflation)
{
	const bool bIsBounded = InParticles.HasBounds(BodyIndex);
	const bool bIsCCD = InParticles.ControlFlags(BodyIndex).GetCCDEnabled();

	if (!bIsCCD && (BoundsVelocityInflation == FReal(0)))
	{
		return FVec3(MinBoundsThickness);
	}

	// See comments in ComputeBoundsThickness<THandle> below
	FReal MaxBoundsThickness = TNumericLimits<FReal>::Max();
	if (bIsBounded)
	{
		if (bIsCCD)
		{
			BoundsVelocityInflation = FMath::Max(FReal(1), BoundsVelocityInflation);
		}
		else
		{
			MaxBoundsThickness = Chaos_Bounds_MaxInflationScale * InParticles.LocalBounds(BodyIndex).Extents().GetMax();
		}
	}

	return ComputeBoundsThickness(InParticles.GetV(BodyIndex), Dt, MinBoundsThickness, MaxBoundsThickness, BoundsVelocityInflation);
}

template <typename THandle>
FVec3 ComputeBoundsThickness(const THandle& ParticleHandle, FReal Dt, FReal MinBoundsThickness, FReal BoundsVelocityInflation)
{
	const typename THandle::FDynamicParticleHandleType* RigidParticle = ParticleHandle.CastToRigidParticle();
	const typename THandle::FKinematicParticleHandleType* KinematicParticle = ParticleHandle.CastToKinematicParticle();
	const bool bIsBounded = ParticleHandle.HasBounds();
	const bool bIsCCD = (RigidParticle != nullptr) && RigidParticle->CCDEnabled();

	if (!bIsCCD && (BoundsVelocityInflation == FReal(0)))
	{
		return FVec3(MinBoundsThickness);
	}

	// Limit the bounds expansion based on the size of the object. This prevents objects that are moved a large
	// distance without resetting physics from having excessive bounds. Objects that move more than their size per
	// tick without CCD enabled will have simulation issues anyway, so expanding bounds beyond this is unnecessary.
	FReal MaxBoundsThickness = TNumericLimits<FReal>::Max();
	if (bIsBounded)
	{
		if (bIsCCD)
		{
			BoundsVelocityInflation = FMath::Max(FReal(1), BoundsVelocityInflation);
		}
		else
		{
			MaxBoundsThickness = Chaos_Bounds_MaxInflationScale * ParticleHandle.LocalBounds().Extents().GetMax();
		}
	}

	FVec3 Vel(0);
	if (KinematicParticle != nullptr)
	{
		Vel = KinematicParticle->V();
	}

	return ComputeBoundsThickness(Vel, Dt, MinBoundsThickness, MaxBoundsThickness, BoundsVelocityInflation);
}

template<class OBJECT_ARRAY>
bool HasBoundingBox(const OBJECT_ARRAY& Objects, const int32 i)
{
	return Objects[i]->HasBoundingBox();
}

template<class T, int d>
bool HasBoundingBox(const TParticles<T, d>& Objects, const int32 i)
{
	return true;
}

template<class T, int d>
bool HasBoundingBox(const TGeometryParticles<T, d>& Objects, const int32 i)
{
	return Objects.GetGeometry(i)->HasBoundingBox();
}

template<class T, int d>
bool HasBoundingBox(const TPBDRigidParticles<T, d>& Objects, const int32 i)
{
	if (Objects.GetGeometry(i))
	{
		return Objects.GetGeometry(i)->HasBoundingBox();
	}
	return Objects.CollisionParticles(i) != nullptr && Objects.CollisionParticles(i)->Size() > 0;
}

template<typename Generic>
bool HasBoundingBox(const Generic& Item)
{
	return Item.HasBoundingBox();
}

template<typename T, int d, bool bPersistent>
bool HasBoundingBox(const TGeometryParticleHandleImp<T, d, bPersistent>& Handle)
{
	return Handle.HasBounds();
}

template<typename T, int d, bool bPersistent>
bool HasBoundingBox(const TPBDRigidParticleHandleImp<T, d, bPersistent>& Handle)
{
	return Handle.HasBounds();
}

template<class OBJECT_ARRAY, class T, int d>
const TAABB<T, d> GetWorldSpaceBoundingBox(const OBJECT_ARRAY& Objects, const int32 i, const TMap<int32, TAABB<T, d>>& WorldSpaceBoxes)
{
	return Objects[i]->BoundingBox();
}

template<class T, int d>
const TAABB<T, d>& GetWorldSpaceBoundingBox(const TParticles<T, d>& Objects, const int32 i, const TMap<int32, TAABB<T, d>>& WorldSpaceBoxes)
{
	return WorldSpaceBoxes[i];
}

template<class T, int d>
const TAABB<T, d>& GetWorldSpaceBoundingBox(const TGeometryParticles<T, d>& Objects, const int32 i, const TMap<int32, TAABB<T, d>>& WorldSpaceBoxes)
{
	return GetWorldSpaceBoundingBox(static_cast<const TParticles<T, d>&>(Objects), i, WorldSpaceBoxes);
}

template<class T, int d>
const TAABB<T, d> GetWorldSpaceBoundingBox(const TPBDRigidParticles<T, d>& Objects, const int32 i, const TMap<int32, TAABB<T, d>>& WorldSpaceBoxes)
{
	return GetWorldSpaceBoundingBox(static_cast<const TParticles<T, d>&>(Objects), i, WorldSpaceBoxes);
}

template<class T, int d>
TAABB<T, d> ComputeWorldSpaceBoundingBox(const TParticles<T, d>& Objects, const int32 i, bool bUseVelocity = false, T Dt = 0)
{
	ensure(!bUseVelocity);
	return TAABB<T, d>(Objects.GetX(i), Objects.GetX(i));
}

template<class T, int d>
TAABB<T, d> ComputeWorldSpaceBoundingBox(const TGeometryParticles<T, d>& Objects, const int32 i, bool bUseVelocity = false, T Dt = 0)
{
	ensure(!bUseVelocity);
	TRigidTransform<T, d> LocalToWorld(Objects.GetX(i), Objects.GetR(i));
	const auto& LocalBoundingBox = Objects.GetGeometry(i)->BoundingBox();
	return LocalBoundingBox.TransformedAABB(LocalToWorld);
}

template<class T, int d>
TAABB<T, d> ComputeWorldSpaceBoundingBox(const TPBDRigidParticles<T, d>& Objects, const int32 i, bool bUseVelocity = false, T Dt = 0)
{
	TRigidTransform<T, d> LocalToWorld(Objects.GetP(i), Objects.GetQ(i));
	TAABB<T, d> WorldSpaceBox;
	if (Objects.GetGeometry(i))
	{
		const auto& LocalBoundingBox = Objects.GetGeometry(i)->BoundingBox();
		WorldSpaceBox = LocalBoundingBox.TransformedAABB(LocalToWorld);
	}
	else
	{
		check(Objects.CollisionParticles(i) && Objects.CollisionParticles(i)->Size());
		TAABB<T, d> LocalBoundingBox(Objects.CollisionParticles(i)->GetX(0), Objects.CollisionParticles(i)->GetX(0));
		for (uint32 j = 1; j < Objects.CollisionParticles(i)->Size(); ++j)
		{
			LocalBoundingBox.GrowToInclude(Objects.CollisionParticles(i)->GetX(j));
		}
		WorldSpaceBox = LocalBoundingBox.TransformedAABB(LocalToWorld);
	}
	
	if (bUseVelocity)
	{
		WorldSpaceBox.ThickenSymmetrically(ComputeBoundsThickness(Objects, Dt, i, 0, 1));
	}
	return WorldSpaceBox;
}

template<typename THandle, typename T, int d, bool bPersistent>
TAABB<T, d> ComputeWorldSpaceBoundingBoxForHandle(const THandle& Handle)
{
	const auto PBDRigid = Handle.CastToRigidParticle();
	const bool bIsRigidDynamic = PBDRigid && PBDRigid->ObjectState() == EObjectStateType::Dynamic;

	TRigidTransform<T, d> LocalToWorld = bIsRigidDynamic ? TRigidTransform<T, d>(PBDRigid->P(), PBDRigid->Q()) : TRigidTransform<T, d>(Handle.X(), Handle.R());
	if(Handle.Geometry())
	{
		const auto& LocalBoundingBox = Handle.Geometry()->BoundingBox();
		return LocalBoundingBox.TransformedBox(LocalToWorld);
	}

	check(PBDRigid);
	check(PBDRigid->CollisionParticles() && PBDRigid->CollisionParticles()->Size());
	TAABB<T, d> LocalBoundingBox(PBDRigid->CollisionParticles()->X(0), PBDRigid->CollisionParticles()->X(0));
	for(uint32 j = 1; j < PBDRigid->CollisionParticles()->Size(); ++j)
	{
		LocalBoundingBox.GrowToInclude(PBDRigid->CollisionParticles()->X(j));
	}
	return LocalBoundingBox.TransformedBox(LocalToWorld);
}

template<typename T, int d, bool bPersistent>
TAABB<T, d> ComputeWorldSpaceBoundingBox(const TGeometryParticleHandleImp<T, d, bPersistent>& Handle, bool bUseVelocity = false, T Dt = 0)
{
	return Handle.WorldSpaceInflatedBounds();
}

template<typename T, int d, bool bPersistent>
TAABB<T, d> ComputeWorldSpaceBoundingBox(const TPBDRigidParticleHandleImp<T, d, bPersistent>& Handle, bool bUseVelocity = false, T Dt = 0)
{
	return Handle.WorldSpaceInflatedBounds();
}

template<typename T, typename GenericEntry>
TAABB<T, 3> ComputeWorldSpaceBoundingBox(const GenericEntry& InEntry, bool bUseVelocity = false, T Dt = 0)
{
	ensure(!bUseVelocity);
	return InEntry.BoundingBox();
}

template<typename OBJECT_ARRAY, typename T, int d>
const TAABB<T, d> ComputeGlobalBoxAndSplitAxis(const OBJECT_ARRAY& Objects, const TArray<int32>& AllObjects, const TMap<int32, TAABB<T, d>>& WorldSpaceBoxes, bool bAllowMultipleSplitting, int32& OutAxis)
{
	TAABB<T, d> GlobalBox = GetWorldSpaceBoundingBox(Objects, AllObjects[0], WorldSpaceBoxes);
	for (int32 i = 1; i < AllObjects.Num(); ++i)
	{
		GlobalBox.GrowToInclude(GetWorldSpaceBoundingBox(Objects, AllObjects[i], WorldSpaceBoxes));
	}
	int32 Axis = 0;
	TVector<T, d> GlobalExtents = GlobalBox.Extents();
	if (GlobalExtents[2] > GlobalExtents[0] && GlobalExtents[2] > GlobalExtents[1])
	{
		Axis = 2;
	}
	else if (GlobalExtents[1] > GlobalExtents[0])
	{
		Axis = 1;
	}
	if (bAllowMultipleSplitting && GlobalExtents[Axis] < GlobalExtents[(Axis + 1) % 3] * 1.25 && GlobalExtents[Axis] < GlobalExtents[(Axis + 2) % 3] * 1.25 && AllObjects.Num() > 4 * MIN_NUM_OBJECTS)
	{
		Axis = -1;
	}

	OutAxis = Axis;
	return GlobalBox;
}

template<typename T, int d>
const TAABB<T, d> ComputeGlobalBoxAndSplitAxis(const TParticles<T,d>& Objects, const TArray<int32>& AllObjects, const TMap<int32, TAABB<T, d>>& WorldSpaceBoxes, bool bAllowMultipleSplitting, int32& OutAxis)
{
	//simple particles means we can split more efficiently
	TPair<int32, int32> Counts[d];

	for (int32 i = 0; i < d; ++i)
	{
		Counts[i].Key = 0;
		Counts[i].Value = 0;
	};

	auto CountLambda = [&](const TVector<T, d>& Point)
	{
		for (int32 i = 0; i < d; ++i)
		{
			Counts[i].Key += Point[i] > 0 ? 0 : 1;
			Counts[i].Value += Point[i] > 0 ? 1 : 0;
		};
	};

	TAABB<T, d> GlobalBox = GetWorldSpaceBoundingBox(Objects, AllObjects[0], WorldSpaceBoxes);
	CountLambda(GlobalBox.Center());
	for (int32 i = 1; i < AllObjects.Num(); ++i)
	{
		TAABB<T, d> PtBox = GetWorldSpaceBoundingBox(Objects, AllObjects[i], WorldSpaceBoxes);
		GlobalBox.GrowToInclude(PtBox);
		CountLambda(PtBox.Center());
	}

	//we pick the axis that gives us the most culled even in the case when it goes in the wrong direction (i.e the biggest min)
	int32 BestAxis = 0;
	int32 MaxCulled = 0;
	for (int32 Axis = 0; Axis < d; ++Axis)
	{
		int32 CulledWorstCase = FMath::Min(Counts[Axis].Key, Counts[Axis].Value);
		if (CulledWorstCase > MaxCulled)
		{
			MaxCulled = CulledWorstCase;
			BestAxis = Axis;
		}
	}
	
	//todo(ocohen): use multi split when CulledWorstCase is similar for every axis

	OutAxis = BestAxis;
	return GlobalBox;
}

template<class OBJECT_ARRAY, class T, int d>
void ComputeAllWorldSpaceBoundingBoxes(const OBJECT_ARRAY& Objects, const TArray<int32>& AllObjects, const bool bUseVelocity, const T Dt, TMap<int32, TAABB<T, d>>& WorldSpaceBoxes)
{
	check(!bUseVelocity);
}

template<class T, int d>
void ComputeAllWorldSpaceBoundingBoxes(const TParticles<T, d>& Objects, const TArray<int32>& AllObjects, const bool bUseVelocity, const T Dt, TMap<int32, TAABB<T, d>>& WorldSpaceBoxes)
{
	check(!bUseVelocity);
	WorldSpaceBoxes.Reserve(AllObjects.Num());
	//PhysicsParallelFor(AllObjects.Num(), [&](int32 i) {
	for (int32 i : AllObjects)
	{
		WorldSpaceBoxes.FindOrAdd(i) = ComputeWorldSpaceBoundingBox(Objects, i);
	}
	//});
}

template<class T, int d>
void ComputeAllWorldSpaceBoundingBoxes(const TGeometryParticles<T, d>& Objects, const TArray<int32>& AllObjects, const bool bUseVelocity, const T Dt, TMap<int32, TAABB<T, d>>& WorldSpaceBoxes)
{
	check(!bUseVelocity);
	WorldSpaceBoxes.Reserve(AllObjects.Num());
	//PhysicsParallelFor(AllObjects.Num(), [&](int32 i) {
	for (int32 i : AllObjects)
	{
		WorldSpaceBoxes.FindOrAdd(i) = ComputeWorldSpaceBoundingBox(Objects, i);
	}
	//});
}

template<class T, int d>
void ComputeAllWorldSpaceBoundingBoxes(const TPBDRigidParticles<T, d>& Objects, const TArray<int32>& AllObjects, const bool bUseVelocity, const T Dt, TMap<int32, TAABB<T, d>>& WorldSpaceBoxes)
{
	WorldSpaceBoxes.Reserve(AllObjects.Num());
	//PhysicsParallelFor(AllObjects.Num(), [&](int32 i) {
	for (int32 i = 0; i < AllObjects.Num(); ++i)
	{
		const int32 BodyIndex = AllObjects[i];
		TAABB<T, d>& WorldSpaceBox = WorldSpaceBoxes.FindOrAdd(BodyIndex);
		WorldSpaceBox = ComputeWorldSpaceBoundingBox(Objects, BodyIndex, bUseVelocity, Dt);
	}
	//});
}

// Tests whether a type is actually a view on particles or just a generic type
// to separate some extended functionality for particle types
struct CParticleView
{
	template<typename T>
	auto Requires() -> decltype(T::THandleType);
};

//todo: how do we protect ourselves and make it const?
template<typename ParticleView, typename T, int d>
typename TEnableIf<TModels_V<CParticleView, ParticleView>>::Type ComputeAllWorldSpaceBoundingBoxes(const ParticleView& Particles, const TArray<bool>& RequiresBounds, const bool bUseVelocity, const T Dt, TArray<TAABB<T, d>>& WorldSpaceBoxes)
{
	WorldSpaceBoxes.AddUninitialized(Particles.Num());
	ParticlesParallelFor(Particles, [&RequiresBounds, &WorldSpaceBoxes, bUseVelocity, Dt](const auto& Particle, int32 Index)
	{
		if (RequiresBounds[Index])
		{
			WorldSpaceBoxes[Index] = ComputeWorldSpaceBoundingBox(Particle);
			if (bUseVelocity)
			{
				if (const auto PBDRigid = Particle.AsDynamic())
				{
					WorldSpaceBoxes[Index].ThickenSymmetrically(ComputeBoundsThickness(*PBDRigid, Dt, 0, 1));
				}
			}
		}
	});
}

template<typename ParticleView, typename T, int d>
typename TEnableIf<!TModels_V<CParticleView, ParticleView>>::Type ComputeAllWorldSpaceBoundingBoxes(const ParticleView& Particles, const TArray<bool>& RequiresBounds, const bool bUseVelocity, const T Dt, TArray<TAABB<T, d>>& WorldSpaceBoxes)
{
	WorldSpaceBoxes.AddUninitialized(Particles.Num());
	ParticlesParallelFor(Particles, [&RequiresBounds, &WorldSpaceBoxes, bUseVelocity, Dt](const auto& Particle, int32 Index)
	{
		if(RequiresBounds[Index])
		{
			WorldSpaceBoxes[Index] = ComputeWorldSpaceBoundingBox(Particle);
		}
	});
}

template<class OBJECT_ARRAY>
int32 GetObjectCount(const OBJECT_ARRAY& Objects)
{
	return Objects.Num();
}

template<class T, int d>
int32 GetObjectCount(const TParticles<T, d>& Objects)
{
	return Objects.Size();
}

template<class T, int d>
int32 GetObjectCount(const TGeometryParticles<T, d>& Objects)
{
	return GetObjectCount(static_cast<const TParticles<T, d>&>(Objects));
}

template<class T, int d>
int32 GetObjectCount(const TPBDRigidParticles<T, d>& Objects)
{
	return GetObjectCount(static_cast<const TParticles<T, d>&>(Objects));
}

template<class OBJECT_ARRAY>
bool IsDisabled(const OBJECT_ARRAY& Objects, const uint32 Index)
{
	return false;
}

template<class T, int d>
bool IsDisabled(const TGeometryParticles<T, d>& Objects, const uint32 Index)
{
	return false;
}

template<class T, int d>
bool IsDisabled(const TPBDRigidParticles<T, d>& Objects, const uint32 Index)
{
	return Objects.Disabled(Index);
}
}
