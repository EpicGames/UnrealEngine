// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Templates/UnrealTemplate.h"

#include "Chaos/ImplicitObject.h"
#include "Chaos/AABB.h"
#include "Chaos/ConvexHalfEdgeStructureData.h"
#include "Chaos/Plane.h"
#include "Chaos/Transform.h"
#include "ChaosArchive.h"
#include "UObject/ExternalPhysicsCustomObjectVersion.h"
#include "UObject/ReleaseObjectVersion.h"

namespace Chaos
{
	/*
	 * Axis-aligned box collision geometry. Consists of a core AABB with a margin. The margin should be considered physically part of the
	 * box - it pads the faces and rounds the corners.
	 */
	template<class T, int d>
	class TBox final : public FImplicitObject
	{
	public:
		using FImplicitObject::SignedDistance;
		using FImplicitObject::GetTypeName;

		FORCEINLINE static constexpr EImplicitObjectType StaticType() { return ImplicitObjectType::Box; }

		// This should never be used outside of creating a default for arrays
		FORCEINLINE TBox()
		    : FImplicitObject(EImplicitObject::FiniteConvex, ImplicitObjectType::Box){};
		FORCEINLINE TBox(const TVector<T, d>& Min, const TVector<T, d>& Max)
		    : FImplicitObject(EImplicitObject::FiniteConvex, ImplicitObjectType::Box)
			, AABB(Min, Max)
		{
			//todo: turn back on
			/*for (int Axis = 0; Axis < d; ++Axis)
			{
				ensure(MMin[Axis] <= MMax[Axis]);
			}*/
		}

		/**
		 * Create a box with the specified size and margin (Min and Max is the desired size including the margin). 
		 */ 
		FORCEINLINE TBox(const TVector<T, d>& InMin, const TVector<T, d>& InMax, FReal InMargin)
			: FImplicitObject(EImplicitObject::FiniteConvex, ImplicitObjectType::Box)
		{
			AABB = TAABB<T, d>(InMin, InMax);
			SetMargin(ClampedMargin(InMargin));
		}

		FORCEINLINE TBox(const TBox<T, d>& Other)
		    : FImplicitObject(EImplicitObject::FiniteConvex, ImplicitObjectType::Box)
			, AABB(Other.AABB)
		{
			SetMargin(Other.GetMarginf());
		}

		FORCEINLINE TBox(TBox<T, d>&& Other)
		    : FImplicitObject(EImplicitObject::FiniteConvex, ImplicitObjectType::Box)
		    , AABB(MoveTemp(Other.AABB))
		{
			SetMargin(Other.GetMarginf());
		}

		FORCEINLINE TBox<T, d>& operator=(const TBox<T, d>& Other)
		{
			this->Type = Other.Type;
			this->bIsConvex = Other.bIsConvex;
			this->bDoCollide = Other.bDoCollide;
			this->bHasBoundingBox = Other.bHasBoundingBox;

			AABB = Other.AABB;
			SetMargin(Other.GetMarginf());
			return *this;
		}

		FORCEINLINE TBox<T, d>& operator=(TBox<T, d>&& Other)
		{
			this->Type = Other.Type;
			this->bIsConvex = Other.bIsConvex;
			this->bDoCollide = Other.bDoCollide;
			this->bHasBoundingBox = Other.bHasBoundingBox;

			AABB = MoveTemp(Other.AABB);
			SetMargin(Other.GetMarginf());
			return *this;
		}

		virtual ~TBox() = default;

		virtual Chaos::FImplicitObjectPtr CopyGeometry() const override
		{
			return Chaos::FImplicitObjectPtr(new TBox<T,d>(*this));
		}

		virtual Chaos::FImplicitObjectPtr CopyGeometryWithScale(const FVec3& Scale) const override
		{
			return Chaos::FImplicitObjectPtr(new TBox<T, d>(AABB.Min() * Scale, AABB.Max() * Scale, Margin * Scale.Min()));
		}

		virtual FReal GetRadius() const override
		{
			return 0.0f;
		}

		/**
		 * Returns sample points centered about the origin.
		 */
		TArray<TVector<T, d>> ComputeLocalSamplePoints() const
		{
			return BoundingBox().ComputeLocalSamplePoints();
		}

		/**
		 * Returns sample points at the current location of the box.
		 */
		TArray<TVector<T, d>> ComputeSamplePoints() const
		{
			return BoundingBox().ComputeSamplePoints();
		}

		FORCEINLINE bool Contains(const TVector<T, d>& Point) const
		{
			return BoundingBox().Contains(Point);
		}

		FORCEINLINE bool Contains(const TVector<T, d>& Point, const T Tolerance) const
		{
			return BoundingBox().Contains(Point, Tolerance);
		}

		// Minimum extents
		FORCEINLINE const TVector<T, d>& Min() const
		{
			return AABB.Min();
		}
		
		// Maximum extents
		FORCEINLINE const TVector<T, d>& Max() const
		{ 
			return AABB.Max();
		}

		// Extents
		FORCEINLINE const TAABB<FReal, 3> BoundingBox() const
		{
			// LWC - necessary if T is different from FReal
			return TAABB<FReal, 3>(AABB);
		}

		// Apply a limit to the specified margin that prevents the box inverting
		FORCEINLINE FReal ClampedMargin(FReal InMargin) const
		{
			FReal MaxMargin = 0.5f * AABB.Extents().Min();
			return FMath::Min(InMargin, MaxMargin);
		}

		// Return the distance and normal is the closest point on the surface to Pos. Negative for penetration.
		virtual FReal PhiWithNormal(const FVec3& Pos, FVec3& Normal) const override
		{
			return AABB.PhiWithNormal(Pos, Normal);
		}

		virtual FReal PhiWithNormalScaled(const FVec3& Pos, const FVec3& Scale, FVec3& Normal) const override
		{
			return TAABB<T, d>(Scale * AABB.Min(), Scale * AABB.Max()).PhiWithNormal(Pos, Normal);
		}

		static FORCEINLINE bool RaycastFast(const TVector<T,d>& InMin, const TVector<T,d>& InMax, const TVector<T, d>& StartPoint, const TVector<T, d>& Dir, const TVector<T, d>& InvDir, const bool* bParallel, const T Length, const T InvLength, T& OutTime, TVector<T, d>& OutPosition)
		{
			return TAABB<T, d>(InMin, InMax).RaycastFast(StartPoint, Dir, InvDir, bParallel, Length, InvLength, OutTime, OutPosition);
		}

		virtual bool Raycast(const FVec3& StartPoint, const FVec3& Dir, const FReal Length, const FReal Thickness, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex) const override
		{
			if (AABB.Raycast(StartPoint, Dir, Length, Thickness, OutTime, OutPosition, OutNormal, OutFaceIndex))
			{
				return true;
			}
			return false;
		}

		TVector<T, d> FindClosestPoint(const TVector<T, d>& StartPoint, const T Thickness = (T)0) const
		{
			return AABB.FindClosestPoint(StartPoint, Thickness);
		}

		virtual Pair<FVec3, bool> FindClosestIntersectionImp(const FVec3& StartPoint, const FVec3& EndPoint, const FReal Thickness) const override
		{
			return AABB.FindClosestIntersectionImp(StartPoint, EndPoint, Thickness);
		}

		virtual FVec3 FindGeometryOpposingNormal(const FVec3& DenormDir, int32 FaceIndex, const FVec3& OriginalNormal) const override
		{
			return AABB.FindGeometryOpposingNormal(DenormDir, FaceIndex, OriginalNormal);
		}

		// Get the index of the plane that most opposes the normal
		int32 GetMostOpposingPlane(const TVector<T, d>& Normal) const
		{
			// NOTE: Index returned here must match indices as set up in SNormals (see Box.cpp)
			int32 AxisIndex = FVec3(Normal.GetAbs()).MaxAxis();
			if (Normal[AxisIndex] < 0.0f)
			{
				AxisIndex += 3;
			}
			return AxisIndex;
		}

		int32 GetMostOpposingPlaneScaled(const TVector<T, d>& Normal, const TVector<T, d>& Scale) const
		{
			// Scale does not affect the face normals of a box
			return GetMostOpposingPlane(Normal);
		}

		// Get the nearest point on an edge and the edge vertices
		// Used for manifold generation
		FVec3 GetClosestEdge(int32 PlaneIndexHint, const FVec3& Position, FVec3& OutEdgePos0, FVec3& OutEdgePos1) const
		{
			TVector<T, d> ClosestEdgePosition = FVec3(0);
			if (PlaneIndexHint >= 0)
			{
				FReal ClosestDistanceSq = FLT_MAX;

				int32 PlaneVerticesNum = NumPlaneVertices(PlaneIndexHint);
				if (PlaneVerticesNum > 0)
				{
					FVec3 P0 = GetVertex(GetPlaneVertex(PlaneIndexHint, PlaneVerticesNum - 1));
					for (int32 PlaneVertexIndex = 0; PlaneVertexIndex < PlaneVerticesNum; ++PlaneVertexIndex)
					{
						const int32 VertexIndex = GetPlaneVertex(PlaneIndexHint, PlaneVertexIndex);
						const TVector<T, d> P1 = GetVertex(VertexIndex);

						const TVector<T, d> EdgePosition = FMath::ClosestPointOnLine(P0, P1, Position);
						const FReal EdgeDistanceSq = (EdgePosition - Position).SizeSquared();

						if (EdgeDistanceSq < ClosestDistanceSq)
						{
							ClosestDistanceSq = EdgeDistanceSq;
							ClosestEdgePosition = EdgePosition;
							OutEdgePos0 = P0;
							OutEdgePos1 = P1;
						}

						P0 = P1;
					}
				}
			}
			else
			{
				// @todo(chaos)
				check(false);
			}
			return ClosestEdgePosition;
		}

		// Get the nearest point on an edge
		TVector<T, d> GetClosestEdgePosition(int32 PlaneIndexHint, const TVector<T, d>& Position) const
		{
			TVector<T, d> Unused0, Unused1;
			return GetClosestEdge(PlaneIndexHint, Position, Unused0, Unused1);
		}

		bool GetClosestEdgeVertices(int32 PlaneIndexHint, const FVec3& Position, int32& OutVertexIndex0, int32& OutVertexIndex1) const
		{
			if (PlaneIndexHint >= 0)
			{
				FReal ClosestDistanceSq = FLT_MAX;

				int32 PlaneVerticesNum = NumPlaneVertices(PlaneIndexHint);
				if (PlaneVerticesNum > 0)
				{
					int32 VertexIndex0 = GetPlaneVertex(PlaneIndexHint, PlaneVerticesNum - 1);
					FVec3 P0 = GetVertex(VertexIndex0);

					for (int32 PlaneVertexIndex = 0; PlaneVertexIndex < PlaneVerticesNum; ++PlaneVertexIndex)
					{
						const int32 VertexIndex1 = GetPlaneVertex(PlaneIndexHint, PlaneVertexIndex);
						const FVec3 P1 = GetVertex(VertexIndex1);

						const TVector<T, d> EdgePosition = FMath::ClosestPointOnLine(P0, P1, Position);
						const FReal EdgeDistanceSq = (EdgePosition - Position).SizeSquared();

						if (EdgeDistanceSq < ClosestDistanceSq)
						{
							OutVertexIndex0 = VertexIndex0;
							OutVertexIndex1 = VertexIndex1;
							ClosestDistanceSq = EdgeDistanceSq;
						}

						VertexIndex0 = VertexIndex1;
						P0 = P1;
					}
					return true;
				}
			}
			else
			{
				// @todo(chaos)
				check(false);
			}
			return false;
		}

		// Get an array of all the plane indices that belong to a vertex (up to MaxVertexPlanes).
		// Returns the number of planes found.
		int32 FindVertexPlanes(int32 VertexIndex, int32* OutVertexPlanes, int32 MaxVertexPlanes) const
		{
			return SStructureData.FindVertexPlanes(VertexIndex, OutVertexPlanes, MaxVertexPlanes);
		}
		
		// Get up to the 3  plane indices that belong to a vertex
		// Returns the number of planes found.
		int32 GetVertexPlanes3(int32 VertexIndex, int32& PlaneIndex0, int32& PlaneIndex1, int32& PlaneIndex2) const
		{
			return SStructureData.GetVertexPlanes3(VertexIndex, PlaneIndex0, PlaneIndex1, PlaneIndex2);
		}

		// The number of vertices that make up the corners of the specified face
		int32 NumPlaneVertices(int32 PlaneIndex) const
		{
			return SStructureData.NumPlaneVertices(PlaneIndex);
		}

		// Get the vertex index of one of the vertices making up the corners of the specified face
		int32 GetPlaneVertex(int32 PlaneIndex, int32 PlaneVertexIndex) const
		{
			return SStructureData.GetPlaneVertex(PlaneIndex, PlaneVertexIndex);
		}

		int32 GetEdgeVertex(int32 EdgeIndex, int32 EdgeVertexIndex) const
		{
			return SStructureData.GetEdgeVertex(EdgeIndex, EdgeVertexIndex);
		}

		int32 GetEdgePlane(int32 EdgeIndex, int32 EdgePlaneIndex) const
		{
			return SStructureData.GetEdgePlane(EdgeIndex, EdgePlaneIndex);
		}

		int32 NumPlanes() const { return SNormals.Num(); }

		int32 NumEdges() const { return SStructureData.NumEdges(); }

		int32 NumVertices() const { return SVertices.Num(); }

		// Get the plane at the specified index (e.g., indices from FindVertexPlanes)
		const TPlaneConcrete<FReal> GetPlane(int32 FaceIndex) const
		{
			const FVec3& PlaneN = SNormals[FaceIndex];
			const FVec3 PlaneX = AABB.Center() + 0.5f * (PlaneN * AABB.Extents());
			return TPlaneConcrete<FReal>(PlaneX, PlaneN);
		}

		void GetPlaneNX(const int32 FaceIndex, FVec3& OutN, FVec3& OutX) const
		{
			OutN = SNormals[FaceIndex];
			OutX = AABB.Center() + 0.5f * (SNormals[FaceIndex] * AABB.Extents());
		}

		// Get the vertex at the specified index (e.g., indices from GetPlaneVertexs)
		const FVec3 GetVertex(int32 VertexIndex) const
		{
			const FVec3& Vertex = SVertices[VertexIndex];
			return AABB.Center() + 0.5f * (Vertex * AABB.Extents());
		}

		// Returns a position on the shape
		FORCEINLINE_DEBUGGABLE TVector<T, d> Support(const TVector<T, d>& Direction, const T Thickness, int32& VertexIndex) const
		{
			return AABB.Support(Direction, Thickness, VertexIndex);
		}

		// Returns a position on the core shape excluding the margin
		FORCEINLINE_DEBUGGABLE TVector<T, d> SupportCore(const TVector<T, d>& Direction, const FReal InMargin, FReal* OutSupportDelta, int32& VertexIndex) const
		{
			return AABB.SupportCore(Direction, InMargin, OutSupportDelta, VertexIndex);
		}

		// Returns a position on the core shape excluding the margin
		FORCEINLINE_DEBUGGABLE VectorRegister4Float SupportCoreSimd(const VectorRegister4Float& Direction, const FReal InMargin) const
		{
			FVec3 DirectionVec3;
			VectorStoreFloat3(Direction, &DirectionVec3);
			int32 VertexIndex = INDEX_NONE;
			FVec3 SupportVert = AABB.SupportCore(DirectionVec3, InMargin, nullptr, VertexIndex);
			return MakeVectorRegisterFloatFromDouble(MakeVectorRegister(SupportVert.X, SupportVert.Y, SupportVert.Z, 0.0));
		}

		FORCEINLINE_DEBUGGABLE TVector<T, d> SupportCoreScaled(const TVector<T, d>& Direction, const FReal InMargin, const TVector<T, d>& Scale, T* OutSupportDelta, int32& VertexIndex) const
		{
			return AABB.SupportCoreScaled(Direction, InMargin, Scale, OutSupportDelta, VertexIndex);
			// @todo(chaos): Needs to operate in scaled space as margin is not non-uniform scalable
			//const FReal InvScale = 1.0f / Scale[0];
			//const FReal NetMargin = InvScale * InMargin;
			//const FVec3 CoreVertex = AABB.SupportCore(Direction * Scale, NetMargin, OutSupportDelta);
			//if (OutSupportDelta != nullptr)
			//{
			//	*OutSupportDelta = *OutSupportDelta * Scale[0];
			//}
			//return CoreVertex * Scale;
		}

		// Returns a winding order multiplier used in the manifold clipping and required when we have negative scales (See ImplicitObjectScaled)
		FORCEINLINE FReal GetWindingOrder() const
		{
			return 1.0f;
		}

		FORCEINLINE TVector<T, d> Center() const { return AABB.Center(); }
		FORCEINLINE TVector<T, d> GetCenter() const { return AABB.GetCenter(); }
		FORCEINLINE TVector<T, d> GetCenterOfMass() const { return AABB.GetCenterOfMass(); }
		FORCEINLINE TVector<T, d> Extents() const { return AABB.Extents(); }

		int LargestAxis() const
		{
			return AABB.LargestAxis();
		}

		// Area of the box
		FORCEINLINE T GetArea() const { return BoundingBox().GetArea(); }

		// Volume of the box
		FORCEINLINE T GetVolume() const { return BoundingBox().GetVolume(); }

		FORCEINLINE PMatrix<T, d, d> GetInertiaTensor(const T Mass) const { return GetInertiaTensor(Mass, Extents()); }
		FORCEINLINE static PMatrix<T, 3, 3> GetInertiaTensor(const T Mass, const TVector<T, 3>& Dim)
		{
			// https://www.wolframalpha.com/input/?i=cuboid
			const T M = Mass / 12;
			const T WW = Dim[0] * Dim[0];
			const T HH = Dim[1] * Dim[1];
			const T DD = Dim[2] * Dim[2];
			return PMatrix<T, 3, 3>(M * (HH + DD), M * (WW + DD), M * (WW + HH));
		}


		FORCEINLINE static TRotation<T, d> GetRotationOfMass()
		{
			return TAABB<T, d>::GetRotationOfMass();
		}

		virtual FString ToString() const override
		{ 
			return FString::Printf(TEXT("Box: Min: [%s], Max: [%s], Margin: %f"), *Min().ToString(), *Max().ToString(), GetMarginf());
		}

		FORCEINLINE void SerializeImp(FArchive& Ar)
		{
			FImplicitObject::SerializeImp(Ar);
			AABB.Serialize(Ar);

			Ar.UsingCustomVersion(FReleaseObjectVersion::GUID);
			if (Ar.CustomVer(FReleaseObjectVersion::GUID) >= FReleaseObjectVersion::MarginAddedToConvexAndBox)
			{
				FRealSingle MarginFloat = (FRealSingle)FImplicitObject::Margin;
				Ar << MarginFloat;
				FImplicitObject::Margin = (FRealSingle)MarginFloat;
			}
		}

		// Some older classes used to use a TBox as a bounding box, but now use a TAABB. However we still need
		// to be able to read the older files, so those older classes should use TBox::SerializeAsAABB and not TAABB::Serialize
		static void SerializeAsAABB(FArchive& Ar, TAABB<T,d>& AABB)
		{
			Ar.UsingCustomVersion(FExternalPhysicsCustomObjectVersion::GUID);
			if (Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) < FExternalPhysicsCustomObjectVersion::TBoxReplacedWithTAABB)
			{
				TBox<T, d> Tmp;
				Ar << Tmp;
				AABB = Tmp.AABB;
			}
			else
			{
				AABB.Serialize(Ar);
			}
		}

		// See comments on SerializeAsAABB
		template<typename OtherType>
		static void SerializeAsAABBs(FArchive& Ar, TArray<TAABB<OtherType, d>>& AABBs)
		{
			Ar.UsingCustomVersion(FExternalPhysicsCustomObjectVersion::GUID);
			if (Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) < FExternalPhysicsCustomObjectVersion::TBoxReplacedWithTAABB)
			{
				TArray<TBox<T, d>> Tmp;
				Ar << Tmp;
				AABBs.Reset(Tmp.Num());
				for (const TBox<T, d>& Box : Tmp)
				{
					AABBs.Emplace(TAABB<OtherType, d>(Box.AABB));
				}
			}
			else if (std::is_same_v<T, OtherType>)
			{
				Ar << AABBs;
			}
			else if (Ar.IsLoading())
			{
				// Reload to a different floating point type
				TArray<TAABB<T, d>> TmpAABBs;
				Ar << TmpAABBs;
				AABBs.Reset(TmpAABBs.Num());
				for (const TAABB<T, d>& TmpAABB : TmpAABBs)
				{
					AABBs.Emplace(TAABB<OtherType, d>(TmpAABB));
				}
			}
			else
			{
				// Serialize from a different floating point type
				TArray<TAABB<T, d>> TmpAABBs;
				TmpAABBs.Reserve(AABBs.Num());
				for (const TAABB<OtherType, d>& AABB : AABBs)
				{
					TmpAABBs.Emplace(TAABB<T, d>(AABB));
				}
				Ar << TmpAABBs;
			}
		}

		// See comments on SerializeAsAABB
		template <typename Key>
		static void SerializeAsAABBs(FArchive& Ar, TMap<Key, TAABB<T, d>>& AABBs)
		{
			Ar.UsingCustomVersion(FExternalPhysicsCustomObjectVersion::GUID);
			if (Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) < FExternalPhysicsCustomObjectVersion::TBoxReplacedWithTAABB)
			{
				TMap<Key,TBox<T, d>> Tmp;
				Ar << Tmp;

				for (const auto& Itr : Tmp)
				{
					AABBs.Add(Itr.Key, Itr.Value.AABB);
				}
			}
			else
			{
				Ar << AABBs;
			}
		}

		virtual void Serialize(FChaosArchive& Ar) override
		{
			FChaosArchiveScopedMemory ScopedMemory(Ar, GetTypeName());
			SerializeImp(Ar);
		}

		virtual void Serialize(FArchive& Ar) override { SerializeImp(Ar); }

		virtual uint32 GetTypeHash() const override 
		{
			return AABB.GetTypeHash();
		}

	private:
		TAABB<T, d> AABB;

		// Structure data shared by all boxes and used for manifold creation
		static TArray<FVec3> SNormals;
		static TArray<FVec3> SVertices;
		static FConvexHalfEdgeStructureDataS16 SStructureData;
	};


	class FBoxFloat3: public FImplicitObject
	{
	public:

		FBoxFloat3()
			: FImplicitObject(EImplicitObject::FiniteConvex, ImplicitObjectType::Box)
		{}

		virtual FReal PhiWithNormal(const FVec3& x, FVec3& Normal) const override
		{
			check(false); // should not be called - empty shell class for legacy serialization
			return 0.;
		}

		FORCEINLINE void SerializeImp(FArchive& Ar)
		{
			FImplicitObject::SerializeImp(Ar);
			Ar << MMin << MMax;

			Ar.UsingCustomVersion(FReleaseObjectVersion::GUID);
			if (Ar.CustomVer(FReleaseObjectVersion::GUID) >= FReleaseObjectVersion::MarginAddedToConvexAndBox)
			{
				Ar << FImplicitObject::Margin;
			}
		}

		virtual void Serialize(FChaosArchive& Ar) override
		{
			FChaosArchiveScopedMemory ScopedMemory(Ar, GetTypeName());
			SerializeImp(Ar);
		}

		virtual void Serialize(FArchive& Ar) override { SerializeImp(Ar); }

		virtual uint32 GetTypeHash() const override
		{
			return HashCombine(Chaos::GetTypeHash(MMin), Chaos::GetTypeHash(MMax));
		}

	private:
		TVector<float, 3> MMin, MMax;
	};
}
