// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "ParticleHandleFwd.h"

namespace Chaos
{
	class FCCDModifier;
	class FCCDModifierAccessor;
	class FCCDModifierParticleIterator;
	class FCCDModifierParticleRange;
	class FPBDCollisionConstraint;

	/*
	 * Allows the user to modify the results of CCD collision detection prior to the CCD rewind being
	 * applied. This is in addition to the midphase modification phase which happens before collision
	 * detection has been run, and the contact modifier phase which happens after we have rewound
	 * 
	 * The CCD "contact" details reported here do not necessarily represent the contact details
	 * that will be used in the contact solving phase. The CCD data provides raw first-touch data from
	 * a swept collision detection test, but we will be rebuilding a full contact manifold at the 
	 * post-CCD position. The CCD normal and position here are not used again. For example, it is not
	 * a good idea to use the CCD data to categorize collisions as wall or floor contacts. If you need
	 * to know the actual contact positions and normals that will be used in the contact reoslution
	 * phase, then see FCollisionContactModifier.
	 * 
	 * @see FMidPhaseModifier, FContactPairModifier
	 */
	class FCCDModifier
	{
	public:
		FCCDModifier()
			: Constraint(nullptr)
		{}

		bool IsValid() const
		{
			return Constraint != nullptr;
		}

		//
		// Accessor functions
		//

		// Whether the two particle actually hit each other in the sweep
		CHAOS_API bool IsSweepHit() const;

		// Get the time of impact. This is as a fraction of total movement and will be in the range [0,1] if a hit was found
		CHAOS_API FReal GetSweepHitTOI() const;

		// Get the sweep impact position for the specified body
		// NOTE: See class comments.
		CHAOS_API FVec3 GetWorldSweepHitLocation(const int32 ParticleIndex) const;

		// Get the sweep impact normal
		// NOTE: See class comments.
		CHAOS_API FVec3 GetWorldSweepHitNormal() const;

		CHAOS_API const FGeometryParticleHandle* GetParticle(const int32 ParticleIndex) const;
		CHAOS_API const FGeometryParticleHandle* GetOtherParticle(const FGeometryParticleHandle* InParticle) const;

		//
		// Modifying functions
		//

		// Re-enable this contact (if it was previously disabled)
		CHAOS_API void Enable();

		// Disable this contact
		CHAOS_API void Disable();

		// Convert this contact to a probe
		CHAOS_API void ConvertToProbe();

	private:
		FCCDModifier(
			FCCDModifierAccessor* InAccessor,
			FPBDCollisionConstraint* InConstraint)
			: Accessor(InAccessor)
			, Constraint(InConstraint)
		{}

		void Reset()
		{
			Accessor = nullptr;
			Constraint = nullptr;
		}

		FCCDModifierAccessor* Accessor;
		FPBDCollisionConstraint* Constraint;

		friend class FCCDModifierParticleIterator;
	};

	/*
	 * Class for iterating over ccd results involving a specific particle
	 */
	class FCCDModifierParticleIterator
	{
	public:
		FCCDModifier& operator*()
		{
			return PairModifier;
		}

		FCCDModifier* operator->()
		{
			return &PairModifier;
		}

		FCCDModifierParticleIterator& operator++()
		{
			return MoveToNext();
		}

		explicit operator bool() const
		{
			return PairModifier.IsValid();
		}

		friend bool operator==(const FCCDModifierParticleIterator& L, const FCCDModifierParticleIterator& R)
		{
			return (L.Range == R.Range) && (L.ConstraintIndex == R.ConstraintIndex);
		}

		friend bool operator!=(const FCCDModifierParticleIterator& L, const FCCDModifierParticleIterator& R)
		{
			return !(L == R);
		}

	private:
		FCCDModifierParticleIterator(FCCDModifierParticleRange* InRange)
			: Range(InRange)
			, ConstraintIndex(INDEX_NONE)
		{ 
		}

		CHAOS_API FCCDModifierParticleIterator& MoveToBegin();
		CHAOS_API FCCDModifierParticleIterator& MoveToEnd();
		CHAOS_API FCCDModifierParticleIterator& MoveToNext();

		FCCDModifierParticleRange* Range;
		int32 ConstraintIndex;
		FCCDModifier PairModifier;

		friend class FCCDModifierParticleRange;
	};

	/*
	 * Represents all the swept constraints on a particle to allow for iteration.
	 */
	class FCCDModifierParticleRange
	{
	public:
		CHAOS_API FCCDModifierParticleRange(
			FCCDModifierAccessor* InAccessor, 
			FGeometryParticleHandle* InParticle);

		CHAOS_API FCCDModifierParticleIterator begin();
		CHAOS_API FCCDModifierParticleIterator end();

	private:
		FCCDModifierAccessor* GetAccessor() const { return Accessor; }
		FGeometryParticleHandle* GetParticle() const { return Particle; }
		int32 GetNumConstraints() const { return Constraints.Num(); }
		FPBDCollisionConstraint* GetConstraint(const int32 Index) const { return Constraints[Index]; }
		CHAOS_API void CollectConstraints();

		FCCDModifierAccessor* Accessor;
		FGeometryParticleHandle* Particle;
		TArray<FPBDCollisionConstraint*> Constraints;

		friend class FCCDModifierParticleIterator;
	};

	/*
	 * Provides access to CCD modifiers
	 */
	class FCCDModifierAccessor
	{
	public:
		CHAOS_API FCCDModifierAccessor(const FReal InDt);

		// Get an object which allows for range iteration over the CCD modifiers for a particle
		CHAOS_API FCCDModifierParticleRange GetModifiers(FGeometryParticleHandle* Particle);

		FReal GetDt() const
		{
			return Dt;
		}

	private:
		FReal Dt;
	};
}
