// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/Collision/PBDCollisionConstraint.h"
#include "Chaos/Collision/CollisionConstraintAllocator.h"
#include "Chaos/ParticleHandleFwd.h"

namespace Chaos
{
	class FPBDCollisionConstraint;
	class FPBDRigidsEvolutionGBF;
	struct FCCDConstraint;

	// FCCDParticle holds data used for applying CCD constraints.
	// We create a FCCDParticle for each dynamic particle involved in CCD constraints, but not for kinematic or static particles.
	// The dynamic particle could be CCD-enabled, or not CCD-enabled but colliding with another CCD-enabled dynamic particle. 
	struct FCCDParticle
	{
		TPBDRigidParticleHandle<FReal, 3>* Particle;
		TArray<FCCDParticle*> OverlappingDynamicParticles; // Dynamic particles that have overlapping (CCD) bounding boxes with this particle's. This is only used for island assignment. This could be optimized by using linked list and putting linked list nodes of all particles into a contiguous array.
		TArray<FCCDConstraint*> AttachedCCDConstraints; // CCDConstraints that refer to this particle. This is used for resweeping.
		int32 Island; // This could be optimized by putting island into a separate contiguous array (SOA vs AOS).
		FReal TOI;
		bool Done; // If a particle is marked as Done == true, we will not apply impulse on it any more in the current frame, effectively treating it as static. This happens when attached CCD constraint reaches maxium process count.

		FCCDParticle()
			: Particle(nullptr)
			, Island(INDEX_NONE)
			, TOI(1.f)
			, Done(false)
		{
			OverlappingDynamicParticles.Reserve(8);
			AttachedCCDConstraints.Reserve(8);
		}

		FCCDParticle(TPBDRigidParticleHandle<FReal, 3>* InParticle)
			: Particle(InParticle)
			, Island(INDEX_NONE)
			, TOI(1.f)
			, Done(false)
		{
			OverlappingDynamicParticles.Reserve(8);
			AttachedCCDConstraints.Reserve(8);
		}

		void AddOverlappingDynamicParticle(FCCDParticle* const InParticle);
		void AddConstraint(FCCDConstraint* const Constraint);
	};

	// FCCDParticle holds data used for applying CCD constraints.
	// We create a FCCDConstraint for each swept constraint.
	struct FCCDConstraint
	{
		FPBDCollisionConstraint* SweptConstraint;
		FCCDParticle* Particle[2];
		int32 Island;
		int32 ProcessedCount; // The number of times this constraint is processed in the current frame.
		int32 FastMovingKinematicIndex;
		FVec3 NetImpulse;

		FCCDConstraint()
			: SweptConstraint(nullptr)
			, Particle{ nullptr, nullptr }
			, Island(INDEX_NONE)
			, ProcessedCount(0)
			, FastMovingKinematicIndex(INDEX_NONE)
			, NetImpulse(0)
		{}

		FCCDConstraint(FPBDCollisionConstraint* const InConstraint, FCCDParticle* InParticle[], const FVec3 Displacements[])
			: SweptConstraint(InConstraint)
			, Particle{ InParticle[0], InParticle[1] }
			, Island(INDEX_NONE)
			, ProcessedCount(0)
			, FastMovingKinematicIndex(GetFastMovingKinematicIndex(InConstraint, Displacements))
			, NetImpulse(0)
		{}

		int32 GetFastMovingKinematicIndex(const FPBDCollisionConstraint* Constraint, const FVec3 Displacements[]) const;
	};

	class FCCDManager
	{
	public:
		FCCDManager(){}

		// Apply the CCD solver to the set of constraints. Currently this just moves the particles so that they overlap by
		// an amount that can be resolved by the standard collision solver.
		void ApplyCCD(const FReal Dt, TArrayView<FPBDCollisionConstraint* const> SweptConstraints, const int32 NumDynamicParticles);

		// Fix up any remaining penetration following the main constraint solver phase. E.g., this prevents heavy objects
		// pushing light CCD objects through the world.
		void ProjectCCD(const FReal Dt, TArrayView<FPBDCollisionConstraint* const> SweptConstraints);

		UE_DEPRECATED(5.6, "No longer functioning. Use ApplyCCD")
		void ApplyConstraintsPhaseCCD(const FReal Dt, Private::FCollisionConstraintAllocator* CollisionAllocator, const int32 NumDynamicParticles = TNumericLimits<int32>::Max()) {}

		UE_DEPRECATED(5.6, "No longer functioning. Use ProjectCCD")
		void ApplyCorrections(const FReal Dt) {}

	private:
		void ApplySweptConstraints(const FReal Dt, TArrayView<FPBDCollisionConstraint* const> SweptConstraints, const int32 NumDynamicParticles = TNumericLimits<int32>::Max());
		bool UpdateParticleSweptConstraints(FCCDParticle* CCDParticle, const FReal IslandTOI, const FReal Dt);
		// This is called after ApplySweptConstraints. This function updates manifold data which will be used in normal solve.
		void UpdateSweptConstraints(const FReal Dt);
		void OverwriteXUsingV(const FReal Dt);
		bool Init(const FReal Dt, TArrayView<FPBDCollisionConstraint* const> SweptConstraints, const int32 NumDynamicParticles = TNumericLimits<int32>::Max());
		void AssignParticleIslandsAndGroupParticles();
		void AssignConstraintIslandsAndRecordConstraintNum();
		void GroupConstraintsWithIslands();
		void ApplyIslandSweptConstraints(const int32 Island, const FReal Dt);
		void ApplyIslandSweptConstraints2(const int32 Island, const FReal Dt);
		void ResetIslandParticles(const int32 Island);
		void ResetIslandConstraints(const int32 Island);
		void AdvanceParticleXToTOI(FCCDParticle *CCDParticle, const FReal TOI, const FReal Dt) const;
		void UpdateParticleP(FCCDParticle *CCDParticle, const FReal Dt) const;
		void ClipParticleP(FCCDParticle *CCDParticle) const;
		void ClipParticleP(FCCDParticle *CCDParticle, const FVec3 Offset) const;
		void ApplyImpulse(FCCDConstraint *CCDConstraint);

		TArray<FCCDParticle> CCDParticles;
		TArray<FCCDParticle*> GroupedCCDParticles; // FCCDParticles are grouped based on their islands.
		TMap<FPBDRigidParticleHandle*, FCCDParticle*> ParticleToCCDParticle; // This map only contains dynamic particles.
		TArray<FCCDConstraint> CCDConstraints;
		TArray<FCCDConstraint*> SortedCCDConstraints; // Constraints are first grouped into islands and then sorted based on TOI.

		int32 IslandNum;
		TArray<FCCDParticle*> IslandStack; // Used to assign islands
		TArray<int32> IslandParticleStart, IslandParticleNum; // IslandParticleNum is the number of dynamic particles in an island.
		TArray<int32> IslandConstraintStart, IslandConstraintNum, IslandConstraintEnd; // The duplication of information in IslandConstraintNum and IslandConstraintEnd is needed when we group constraints using islands.
	};

	struct FCCDHelpers
	{
		static FRigidTransform3 GetParticleTransformAtTOI(const FGeometryParticleHandle* Particle, const FReal TOI, const FReal Dt);

		// Return true if DeltaX indicates a movement beyond a set of threshold
		// distances for each local axis.
		static bool DeltaExceedsThreshold(const FVec3& AxisThreshold, const FVec3& DeltaX, const FQuat& R);
		static bool DeltaExceedsThreshold(const FVec3& AxisThreshold, const FVec3& DeltaX, const FQuat& R, FVec3& OutAbsLocalDelta, FVec3& AxisThresholdScaled, FVec3& AxisThresholdDiff);

		// Return true if a pair of DeltaXs potentially indicate closing
		// motion between two bodies which would cause the crossing of a combined
		// threshold on any local axis.
		static bool DeltaExceedsThreshold(
			const FVec3& AxisThreshold0, const FVec3& DeltaX0, const FQuat& R0,
			const FVec3& AxisThreshold1, const FVec3& DeltaX1, const FQuat& R1);

		// Variant of the two-particle of DeltaExceedsThreshold which chooses
		// DeltaX: For rigid particles, use X-P. For non-rigids, use Vec3::ZeroVector.
		// R: For rigid particles, use Q. For non-rigids, use R.
		static bool DeltaExceedsThreshold(const FGeometryParticleHandle& Particle0, const FGeometryParticleHandle& Particle1);

		// Variant of the two-particle of DeltaExceedsThreshold which chooses
		// DeltaX: For rigid particles, use V*Dt. For non-rigids, use Vec3::ZeroVector.
		// R: For rigid particles, use Q. For non-rigids, use R.
		static bool DeltaExceedsThreshold(const FGeometryParticleHandle& Particle0, const FGeometryParticleHandle& Particle1, const FReal Dt);
	};


	using CCDHelpers UE_DEPRECATED(5.3, "Renamed to FCCDHelpers to meet naming conventions") = FCCDHelpers;
}
