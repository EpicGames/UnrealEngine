// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Cylinder.h"
#include "Chaos/GJKShape.h"
#include "Chaos/ImplicitFwd.h"
#include "Chaos/ImplicitObject.h"
#include "Chaos/ImplicitObjectUnion.h"
#include "Chaos/Raycasts.h"
#include "Chaos/Sphere.h"
#include "Chaos/Segment.h"
#include "ChaosArchive.h"

#include "Math/VectorRegister.h"

#include "UObject/ReleaseObjectVersion.h"

namespace Chaos
{
	struct FCapsuleSpecializeSamplingHelper;

	class FCapsule final : public FImplicitObject
	{
	public:
		using FImplicitObject::SignedDistance;
		using FImplicitObject::GetTypeName;

		FCapsule()
		    : FImplicitObject(EImplicitObject::FiniteConvex, ImplicitObjectType::Capsule)
		{}
		FCapsule(const FVec3& x1, const FVec3& x2, const FReal Radius)
		    : FImplicitObject(EImplicitObject::FiniteConvex, ImplicitObjectType::Capsule)
			, MSegment(x1, x2)
		{
			SetRadius(static_cast<FRealSingle>(Radius));
		}

		FCapsule(const FCapsule& Other)
		    : FImplicitObject(EImplicitObject::FiniteConvex, ImplicitObjectType::Capsule)
			, MSegment(Other.MSegment)
		{
			SetRadius(Other.GetRadiusf());
		}

		FCapsule(FCapsule&& Other)
		    : FImplicitObject(EImplicitObject::FiniteConvex, ImplicitObjectType::Capsule)
			, MSegment(MoveTemp(Other.MSegment))
		{
			SetRadius(Other.GetRadiusf());
		}

		FCapsule& operator=(FCapsule&& InSteal)
		{
			this->Type = InSteal.Type;
			this->bIsConvex = InSteal.bIsConvex;
			this->bDoCollide = InSteal.bDoCollide;
			this->bHasBoundingBox = InSteal.bHasBoundingBox;

			MSegment = MoveTemp(InSteal.MSegment);
			SetRadius(InSteal.GetRadiusf());

			return *this;
		}

		~FCapsule() = default;

		static constexpr EImplicitObjectType StaticType() { return ImplicitObjectType::Capsule; }

		static FCapsule NewFromOriginAndAxis(const FVec3& Origin, const FVec3& Axis, const FReal Height, const FReal Radius)
		{
			auto X1 = Origin + Axis * Radius;
			auto X2 = Origin + Axis * (Radius + Height);
			return FCapsule(X1, X2, Radius);
		}

		UE_DEPRECATED(5.6, "Please Use GetRadiusf instead.")
		virtual FReal GetRadius() const override
		{
			return static_cast<FReal>(Margin);
		}

		virtual FRealSingle GetRadiusf() const override
		{
			return Margin;
		}

		/**
		 * Returns sample points centered about the origin.
		 *
		 * \p NumPoints specifies how many points to generate.
		 */
		TArray<FVec3> ComputeLocalSamplePoints(const int32 NumPoints) const;

		/** 
		 * Returns sample points centered about the origin. 
		 *
		 * \p PointsPerUnitArea specifies how many points to generate per square 
		 *    unit (cm). 0.5 would generate 1 point per 2 square cm.
		 */
		TArray<FVec3> ComputeLocalSamplePoints(const FReal PointsPerUnitArea, const int32 MinPoints = 0, const int32 MaxPoints = 1000) const
		{ return ComputeLocalSamplePoints(FMath::Clamp(static_cast<int32>(ceil(PointsPerUnitArea * GetArea())), MinPoints, MaxPoints)); }

		/**
		 * Returns sample points at the current location of the cylinder.
		 */
		TArray<FVec3> ComputeSamplePoints(const int32 NumPoints) const;

		/** 
		 * Returns sample points at the current location of the cylinder.
		 *
		 * \p PointsPerUnitArea specifies how many points to generate per square 
		 *    unit (cm). 0.5 would generate 1 point per 2 square cm.
		 */
		TArray<FVec3> ComputeSamplePoints(const FReal PointsPerUnitArea, const int32 MinPoints = 0, const int32 MaxPoints = 1000) const
		{ return ComputeSamplePoints(FMath::Clamp(static_cast<int32>(ceil(PointsPerUnitArea * GetArea())), MinPoints, MaxPoints)); }

		virtual FReal PhiWithNormal(const FVec3& x, FVec3& Normal) const override
		{
			FVec3 xf = x;
			FRealSingle Dot = FMath::Clamp(FVec3f::DotProduct(xf - GetX1f(), GetAxisf()), 0.0f, GetHeightf());
			FVec3f ProjectedPoint = Dot * GetAxisf() + GetX1f();
			Normal = xf - ProjectedPoint;
			return Normal.SafeNormalize() - GetRadiusf();
		}

		virtual const FAABB3 BoundingBox() const override
		{
			FAABB3 Box = FAABB3(MSegment.BoundingBox());
			Box.Thicken(GetRadiusf());
			return Box;
		}

		virtual FAABB3 CalculateTransformedBounds(const FRigidTransform3& Transform) const override
		{
			const FVec3 X1 = Transform.TransformPositionNoScale(FVec3(MSegment.GetX1()));
			const FVec3 X2 = Transform.TransformPositionNoScale(FVec3(MSegment.GetX2()));
			const FVec3 MinSegment = X1.ComponentwiseMin(X2);
			const FVec3 MaxSegment = X1.ComponentwiseMax(X2);

			const FVec3 RadiusV = FVec3(GetRadiusf());
			return FAABB3(MinSegment - RadiusV, MaxSegment + RadiusV);
		}

		static bool RaycastFast(FReal MRadius, FReal MHeight, const FVec3& MVector, const FVec3& X1, const FVec3& X2, const FVec3& StartPoint, const FVec3& Dir, const FReal Length, const FReal Thickness, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex)
		{
			OutFaceIndex = INDEX_NONE;
			return Raycasts::RayCapsule(StartPoint, Dir, Length, Thickness, MRadius, MHeight, MVector, X1, X2, OutTime, OutPosition, OutNormal);
		}

		virtual bool Raycast(const FVec3& StartPoint, const FVec3& Dir, const FReal Length, const FReal Thickness, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex) const override
		{
			return RaycastFast(GetRadiusf(), GetHeightf(), GetAxis(), GetX1f(), GetX2f(), StartPoint, Dir, Length, Thickness, OutTime, OutPosition, OutNormal, OutFaceIndex);
		}

		FORCEINLINE FVec3 Support(const FVec3& Direction, const FReal Thickness, int32& VertexIndex) const
		{
			return MSegment.Support(FVec3f(Direction), GetRadiusf() + FRealSingle(Thickness), VertexIndex);
		}

		FORCEINLINE FVec3f Supportf(const FVec3f& Direction, const FRealSingle Thickness, int32& VertexIndex) const
		{
			return MSegment.Support(Direction, GetRadiusf() + Thickness, VertexIndex);
		}

		FORCEINLINE FVec3 SupportCore(const FVec3& Direction, const FReal InMargin, FReal* OutSupportDelta, int32& VertexIndex) const
		{
			// NOTE: Ignores InMargin, assumes Radius
			return MSegment.SupportCore(FVec3f(Direction), VertexIndex);
		}

		FORCEINLINE FVec3f SupportCore(const FVec3f& Direction, const FRealSingle InMargin, FRealSingle* OutSupportDelta, int32& VertexIndex) const
		{
			// NOTE: Ignores InMargin, assumes Radius
			return MSegment.SupportCore(Direction, VertexIndex);
		}

		FORCEINLINE VectorRegister4Float SupportCoreSimd(const VectorRegister4Float& Direction, const FReal InMargin) const
		{
			// NOTE: Ignores InMargin, assumes Radius
			FVec3 DirectionVec3;
			VectorStoreFloat3(Direction, &DirectionVec3);
			int32 VertexIndex = INDEX_NONE;
			FVec3 SupportVert =  MSegment.SupportCore(DirectionVec3, VertexIndex);
			return MakeVectorRegisterFloatFromDouble(MakeVectorRegister(SupportVert.X, SupportVert.Y, SupportVert.Z, 0.0));
		}


		FORCEINLINE FVec3 SupportCoreScaled(const FVec3& Direction, const FReal InMargin, const FVec3& Scale, FReal* OutSupportDelta, int32& VertexIndex) const
		{
			// NOTE: Ignores InMargin, assumes Radius
			// Note: Scaling the direction vector like this, might not seem quite right, but works due to the commutativity of the single dot product that follows
			FRealSingle SupportDeltaSingle;
			const FVec3 Result = SupportCore(Scale * Direction, GetMarginf(), &SupportDeltaSingle, VertexIndex) * Scale;
			if (OutSupportDelta)
			{
				*OutSupportDelta = SupportDeltaSingle;
			}
			return Result;
		}

		FORCEINLINE void SerializeImp(FArchive& Ar)
		{
			Ar.UsingCustomVersion(FExternalPhysicsCustomObjectVersion::GUID);
			FImplicitObject::SerializeImp(Ar);
			MSegment.Serialize(Ar);

			// Radius is now stored in the base class Margin
			FRealSingle ArRadius = GetRadiusf();
			Ar << ArRadius;
			SetRadius(ArRadius);
			
			if(Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) < FExternalPhysicsCustomObjectVersion::CapsulesNoUnionOrAABBs)
			{
				FAABB3 DummyBox;	//no longer store this, computed on demand
				TBox<FReal,3>::SerializeAsAABB(Ar,DummyBox);
			}
		}

		virtual void Serialize(FChaosArchive& Ar) override
		{
			Ar.UsingCustomVersion(FExternalPhysicsCustomObjectVersion::GUID);
			FChaosArchiveScopedMemory ScopedMemory(Ar, GetTypeName());
			SerializeImp(Ar);

			if(Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) < FExternalPhysicsCustomObjectVersion::CapsulesNoUnionOrAABBs)
			{
				TUniquePtr<FImplicitObjectUnion> TmpUnion;
				Ar << TmpUnion;
			}
		}

		virtual FString ToString() const override
		{
			return FString::Printf(TEXT("Capsule: Height: %f Radius: %f"), GetHeightf(), GetRadiusf());
		}
		
		virtual Chaos::FImplicitObjectPtr CopyGeometry() const override
		{
			return Chaos::FImplicitObjectPtr(new FCapsule(*this));
		}

		virtual Chaos::FImplicitObjectPtr CopyGeometryWithScale(const FVec3& Scale) const override
		{
			return  Chaos::FImplicitObjectPtr(new FCapsule(GetX1f() * Scale, GetX2f() * Scale, GetRadiusf() * Scale.Min()));
		}

		UE_DEPRECATED(5.6, "Use GetHeightf instead")
		FReal GetHeight() const { return FReal(MSegment.GetLength()); }
		FRealSingle GetHeightf() const { return MSegment.GetLength(); }
		/** Returns the bottommost point on the capsule. */
		UE_DEPRECATED(5.6, "Use GetOriginf instead")
		const FVec3 GetOrigin() const { return GetX1f() + GetAxisf() * -GetRadiusf(); }
		const FVec3f GetOriginf() const { return GetX1f() + GetAxisf() * -GetRadiusf(); }
		/** Returns the topmost point on the capsule. */
		const FVec3f GetInsertion() const { return GetX1f() + GetAxisf() * (GetHeightf() + GetRadiusf()); }
		UE_DEPRECATED(5.6, "Use GetCenterf instead")
		FVec3 GetCenter() const { return MSegment.GetCenter(); }
		FVec3f GetCenterf() const { return MSegment.GetCenter(); }
		/** Returns the centroid (center of mass). */
		FVec3 GetCenterOfMass() const { return GetCenterf(); }
		FVec3f GetCenterOfMassf() const { return GetCenterf(); }
		const FVec3 GetAxis() const { return MSegment.GetAxis(); }
		UE_DEPRECATED(5.6, "Use GetX1f instead")
		const FVec3 GetX1() const { return MSegment.GetX1(); }
		UE_DEPRECATED(5.6, "Use GetX2f instead")
		FVec3 GetX2() const { return MSegment.GetX2(); }
		const FVec3f GetAxisf() const { return MSegment.GetAxis(); }
		const FVec3f GetX1f() const { return MSegment.GetX1(); }
		FVec3f GetX2f() const { return MSegment.GetX2(); }
		TSegment<FReal> GetSegment() const { return TSegment<FReal>(GetX1f(), GetX2f()); }

		FReal GetArea() const { return GetArea(GetHeightf(), GetRadiusf()); }
		static FReal GetArea(const FReal Height, const FReal Radius)
		{
			static const FReal PI2 = 2.f * UE_PI;
			return PI2 * Radius * (Height + 2.f * Radius); 
		}

		FReal GetVolume() const { return GetVolume(GetHeightf(), GetRadiusf()); }
		static FReal GetVolume(const FReal Height, const FReal Radius) { static const FReal FourThirds = 4.0f / 3.0f; return UE_PI * Radius * Radius * (Height + FourThirds * Radius); }

		FMatrix33 GetInertiaTensor(const FReal Mass) const { return GetInertiaTensor(Mass, GetHeightf(), GetRadiusf()); }
		static FMatrix33 GetInertiaTensor(const FReal Mass, const FReal Height, const FReal Radius)
		{
			// https://www.wolframalpha.com/input/?i=capsule&assumption=%7B%22C%22,+%22capsule%22%7D+-%3E+%7B%22Solid%22%7D
			const FReal R = FMath::Clamp(Radius, (FReal)0., TNumericLimits<FReal>::Max());
			const FReal H = FMath::Clamp(Height, (FReal)0., TNumericLimits<FReal>::Max());
			const FReal RR = R * R;
			const FReal HH = H * H;

			// (5H^3 + 20*H^2R + 45HR^2 + 32R^3) / (60H + 80R)
			const FReal Diag12 = static_cast<FReal>(Mass * (5.*HH*H + 20.*HH*R + 45.*H*RR + 32.*RR*R) / (60.*H + 80.*R));
			// (R^2 * (15H + 16R) / (30H +40R))
			const FReal Diag3 = static_cast<FReal>(Mass * (RR * (15.*H + 16.*R)) / (30.*H + 40.*R));

			return FMatrix33(Diag12, Diag12, Diag3);
		}

		FRotation3 GetRotationOfMass() const { return GetRotationOfMass(GetAxis()); }
		static FRotation3 GetRotationOfMass(const FVec3& Axis)
		{
			// since the capsule stores an axis and the InertiaTensor is assumed to be along the ZAxis
			// we need to make sure to return the rotation of the axis from Z
			return FRotation3::FromRotatedVector(FVec3(0, 0, 1), Axis);
		}

		virtual uint32 GetTypeHash() const override
		{
			return HashCombine(UE::Math::GetTypeHash(GetX1f()), UE::Math::GetTypeHash(GetAxis()));
		}

		FVec3 GetClosestEdgePosition(int32 PlaneIndexHint, const FVec3& Position) const
		{
			FVec3 P0 = GetX1f();
			FVec3 P1 = GetX2f();
			const FVec3 EdgePosition = FMath::ClosestPointOnLine(P0, P1, Position);
			return EdgePosition;
		}
		

		// The number of vertices that make up the corners of the specified face
		// In the case of a capsule the segment will act as a degenerate face
		// Used for manifold generation
		int32 NumPlaneVertices(int32 PlaneIndex) const
		{
			return 2;
		}

		// Returns a winding order multiplier used in the manifold clipping and required when we have negative scales (See ImplicitObjectScaled)
		// Not used for capsules
		// Used for manifold generation
		FORCEINLINE FReal GetWindingOrder() const
		{
			ensure(false);
			return 1.0f;
		}

		// Get the vertex at the specified index (e.g., indices from GetPlaneVertexs)
		// Used for manifold generation
		const FVec3 GetVertex(int32 VertexIndex) const
		{
			FVec3 Result(0);

			switch (VertexIndex)
			{
			case 0:
				Result = GetX1f(); break;
			case 1:
				Result = GetX2f(); break;
			}

			return Result;
		}

		// Get the index of the plane that most opposes the normal
		// not applicable for capsules
		int32 GetMostOpposingPlane(const FVec3& Normal) const
		{
			return 0;
		}

		int32 GetMostOpposingPlaneScaled(const FVec3& Normal, const FVec3& Scale) const
		{
			return 0;
		}

		// Get the vertex index of one of the vertices making up the corners of the specified face
		// Used for manifold generation
		int32 GetPlaneVertex(int32 PlaneIndex, int32 PlaneVertexIndex) const
		{
			return PlaneVertexIndex;
		}

		// Get the plane at the specified index (e.g., indices from FindVertexPlanes)
		const TPlaneConcrete<FReal, 3> GetPlane(int32 FaceIndex) const
		{
			return TPlaneConcrete<FReal, 3>(FVec3(0), FVec3(0));
		}

		void GetPlaneNX(const int32 FaceIndex, FVec3& OutN, FVec3& OutX) const
		{
			OutN = FVec3(0);
			OutX = FVec3(0);
		}

		// Get an array of all the plane indices that belong to a vertex (up to MaxVertexPlanes).
		// Returns the number of planes found.
		int32 FindVertexPlanes(int32 VertexIndex, int32* OutVertexPlanes, int32 MaxVertexPlanes) const
		{
			return 0; 
		}
		
		// Get up to the 3  plane indices that belong to a vertex
		// Returns the number of planes found.
		int32 GetVertexPlanes3(int32 VertexIndex, int32& PlaneIndex0, int32& PlaneIndex1, int32& PlaneIndex2) const
		{
			return 0;
		}

		// Capsules have no planes
		// Used for manifold generation
		int32 NumPlanes() const { return 0; }

#if INTEL_ISPC
		// See PerParticlePBDCollisionConstraint.cpp
		// ISPC code has matching structs for interpreting FImplicitObjects.
		// This is used to verify that the structs stay the same.
		struct FISPCDataVerifier
		{
			static constexpr int32 OffsetOfMSegment() { return offsetof(FCapsule, MSegment); }
			static constexpr int32 SizeOfMSegment() { return sizeof(FCapsule::MSegment); }
		};
		friend FISPCDataVerifier;
#endif // #if INTEL_ISPC

	private:
		void SetRadius(FRealSingle InRadius) { SetMargin(InRadius); }

		TSegment<FRealSingle> MSegment;
	};

	struct FCapsuleSpecializeSamplingHelper
	{
		static FORCEINLINE void ComputeSamplePoints(TArray<FVec3>& Points, const FCapsule& Capsule, const int32 NumPoints)
		{
			if (NumPoints <= 1 || Capsule.GetRadiusf() <= UE_SMALL_NUMBER)
			{
				const int32 Offset = Points.Num();
				if (Capsule.GetHeightf() <= UE_SMALL_NUMBER)
				{
					Points.SetNumUninitialized(Offset + 1);
					Points[Offset] = Capsule.GetCenterf();
				}
				else
				{
					Points.SetNumUninitialized(Offset + 3);
					Points[0] = Capsule.GetOriginf();
					Points[1] = Capsule.GetCenterf();
					Points[2] = Capsule.GetInsertion();
				}
				return;
			}
			ComputeGoldenSpiralPoints(Points, Capsule, NumPoints);
		}

		static FORCEINLINE void ComputeGoldenSpiralPoints(TArray<FVec3>& Points, const FCapsule& Capsule, const int32 NumPoints)
		{ ComputeGoldenSpiralPoints(Points, Capsule.GetOriginf(), Capsule.GetAxisf(), Capsule.GetHeightf(), Capsule.GetRadiusf(), NumPoints); }

		static FORCEINLINE void ComputeGoldenSpiralPoints(
		    TArray<FVec3>& Points,
		    const FVec3& Origin,
		    const FVec3& Axis,
		    const FReal Height,
		    const FReal Radius,
		    const int32 NumPoints)
		{
			// Axis should be normalized.
			checkSlow(FMath::Abs(Axis.Size() - 1.0) < UE_KINDA_SMALL_NUMBER);

			// Evenly distribute points between the capsule body and the end caps.
			int32 NumPointsEndCap;
			int32 NumPointsCylinder;
			const FReal CapArea = 4 * UE_PI * Radius * Radius;
			const FReal CylArea = static_cast<FReal>(2.0 * UE_PI * Radius * Height);
			if (CylArea > UE_KINDA_SMALL_NUMBER)
			{
				const FReal AllArea = CylArea + CapArea;
				NumPointsCylinder = static_cast<int32>(round(CylArea / AllArea * static_cast<FReal>(NumPoints)));
				NumPointsCylinder += (NumPoints - NumPointsCylinder) % 2;
				NumPointsEndCap = (NumPoints - NumPointsCylinder) / 2;
			}
			else
			{
				NumPointsCylinder = 0;
				NumPointsEndCap = (NumPoints - (NumPoints % 2)) / 2;
			}
			const int32 NumPointsToAdd = NumPointsCylinder + NumPointsEndCap * 2;
			Points.Reserve(Points.Num() + NumPointsToAdd);

			const int32 Offset = Points.Num();
			const FReal HalfHeight = Height / 2;
			{
				// Points vary in Z: [-Radius-HalfHeight, -HalfHeight]
				TSphereSpecializeSamplingHelper<FReal, 3>::ComputeBottomHalfSemiSphere(
				    Points, FSphere(FVec3(0, 0, -HalfHeight), Radius), NumPointsEndCap, Points.Num());
#if 0
				{
					FSphere Sphere(FVec3(0, 0, -HalfHeight), Radius);
					for(int32 i=Offset; i < Points.Num(); i++)
					{
						const FVec3& Pt = Points[i];
						const FReal Phi = Sphere.SignedDistance(Pt);
						checkSlow(FMath::Abs(Phi) < KINDA_SMALL_NUMBER);
						checkSlow(Pt[2] > -Radius - HalfHeight - KINDA_SMALL_NUMBER && Pt[2] < -HalfHeight + KINDA_SMALL_NUMBER);
					}
				}
#endif
				// Points vary in Z: [-HalfHeight, HalfHeight], about the Z axis.
				FCylinderSpecializeSamplingHelper::ComputeGoldenSpiralPointsUnoriented(
				    Points, Radius, Height, NumPointsCylinder, false, Points.Num());
#if 0
				{
					TCylinder<FReal> Cylinder(FVec3(0, 0, -HalfHeight), FVec3(0, 0, HalfHeight), Radius);
					for(int32 i=TmpOffset; i < Points.Num(); i++)
					{
						const FVec3& Pt = Points[i];
						const FReal Phi = Cylinder.SignedDistance(Pt);
						checkSlow(FMath::Abs(Phi) < KINDA_SMALL_NUMBER);
						checkSlow(Pt[2] > -HalfHeight - KINDA_SMALL_NUMBER && Pt[2] < HalfHeight + KINDA_SMALL_NUMBER);
					}
				}
#endif
				// Points vary in Z: [HalfHeight, HalfHeight+Radius]
				TSphereSpecializeSamplingHelper<FReal, 3>::ComputeTopHalfSemiSphere(
				    Points, FSphere(FVec3(0, 0, HalfHeight), Radius), NumPointsEndCap, Points.Num());
#if 0
				{
					FSphere Sphere(FVec3(0, 0, HalfHeight), Radius);
					for(int32 i=TmpOffset; i < Points.Num(); i++)
					{
						const FVec3& Pt = Points[i];
						const FReal Phi = Sphere.SignedDistance(Pt);
						checkSlow(FMath::Abs(Phi) < KINDA_SMALL_NUMBER);
						checkSlow(Pt[2] > HalfHeight - KINDA_SMALL_NUMBER && Pt[2] < HalfHeight + Radius + KINDA_SMALL_NUMBER);
					}
				}
#endif
#if 0
				{
					FCapsule(FVec3(0, 0, -HalfHeight), FVec3(0, 0, HalfHeight), Radius);
					for(int32 i=Offset; i < Points.Num(); i++)
					{
						const FVec3& Pt = Points[i];
						const FReal Phi = Cylinder.SignedDistance(Pt);
						checkSlow(FMath::Abs(Phi) < KINDA_SMALL_NUMBER);
					}
				}
#endif
			}

			const FRotation3 Rotation = FRotation3::FromRotatedVector(FVec3(0, 0, 1), Axis);
			checkSlow(((Origin + Axis * (Height + Radius * 2)) - (Rotation.RotateVector(FVec3(0, 0, Height + Radius * 2)) + Origin)).Size() < UE_KINDA_SMALL_NUMBER);
			for (int32 i = Offset; i < Points.Num(); i++)
			{
				FVec3& Point = Points[i];
				const FVec3 PointNew = Rotation.RotateVector(Point + FVec3(0, 0, HalfHeight + Radius)) + Origin;
				checkSlow(FMath::Abs(FCapsule::NewFromOriginAndAxis(Origin, Axis, Height, Radius).SignedDistance(PointNew)) < UE_KINDA_SMALL_NUMBER);
				Point = PointNew;
			}
		}
	};

	FORCEINLINE TArray<FVec3> FCapsule::ComputeLocalSamplePoints(const int32 NumPoints) const
	{
		TArray<FVec3> Points;
		const FVec3f Mid = GetCenterf();
		const FCapsule Capsule(GetX1f() - Mid, GetX1f() + (GetAxisf() * GetHeightf()) - Mid, GetRadiusf());
		FCapsuleSpecializeSamplingHelper::ComputeSamplePoints(Points, Capsule, NumPoints);
		return Points;
	}

	FORCEINLINE TArray<FVec3> FCapsule::ComputeSamplePoints(const int32 NumPoints) const
	{
		TArray<FVec3> Points;
		FCapsuleSpecializeSamplingHelper::ComputeSamplePoints(Points, *this, NumPoints);
		return Points;
	}

	template<class T>
	using TCapsule = FCapsule; // AABB<> is still using TCapsule<> so no deprecation message for now

	template<class T>
	using TCapsuleSpecializeSamplingHelper UE_DEPRECATED(4.27, "Deprecated. this class is to be deleted, use FCapsuleSpecializeSamplingHelper instead") = FCapsuleSpecializeSamplingHelper;

} // namespace Chaos
