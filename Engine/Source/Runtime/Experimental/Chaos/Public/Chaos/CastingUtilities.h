// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/ImplicitObject.h"

#include "Chaos/ImplicitObjectScaled.h"
#include "Chaos/ImplicitObjectTransformed.h"
#include "Chaos/ImplicitObjectUnion.h"
#include "Chaos/Sphere.h"
#include "Chaos/Capsule.h"
#include "Chaos/Convex.h"
#include "Chaos/TriangleMeshImplicitObject.h"
#include "Chaos/Box.h"

namespace Chaos
{
	namespace Utilities
	{
		// Call the lambda with concrete shape type. Unwraps shapes contained in Instanced (e.g., Instanced-Sphere will be called with Sphere. Note that this will effectively discard any instance properties like the margin)
		template <typename Lambda>
		FORCEINLINE_DEBUGGABLE decltype(auto) CastHelper(const FImplicitObject& Geom, const Lambda& Func)
		{
			const EImplicitObjectType Type = Geom.GetType();
			switch (Type)
			{
			case ImplicitObjectType::Sphere:
			{
				return Func(Geom.template GetObjectChecked<FSphere>());
			}
			case ImplicitObjectType::Box:
			{
				return Func(Geom.template GetObjectChecked<TBox<FReal, 3>>());
			}
			case ImplicitObjectType::Capsule:
			{
				return Func(Geom.template GetObjectChecked<FCapsule>());
			}
			case ImplicitObjectType::Convex:
			{
				return Func(Geom.template GetObjectChecked<FConvex>());
			}
			case ImplicitObjectType::IsScaled | ImplicitObjectType::Sphere:
			{
				return Func(Geom.template GetObjectChecked<TImplicitObjectScaled<FSphere>>());
			}
			case ImplicitObjectType::IsScaled | ImplicitObjectType::Box:
			{
				return Func(Geom.template GetObjectChecked< TImplicitObjectScaled<TBox<FReal, 3>>>());
			}
			case ImplicitObjectType::IsScaled | ImplicitObjectType::Capsule:
			{
				return Func(Geom.template GetObjectChecked< TImplicitObjectScaled<FCapsule>>());
			}
			case ImplicitObjectType::IsScaled | ImplicitObjectType::Convex:
			{
				return Func(Geom.template GetObjectChecked< TImplicitObjectScaled<FConvex>>());
			}
			case ImplicitObjectType::IsInstanced | ImplicitObjectType::Sphere:
			{
				return Func(Geom.template GetObjectChecked< TImplicitObjectInstanced<FSphere>>().GetInstancedObject()->template GetObjectChecked<FSphere>());
			}
			case ImplicitObjectType::IsInstanced | ImplicitObjectType::Box:
			{
				return Func(Geom.template GetObjectChecked< TImplicitObjectInstanced<TBox<FReal, 3>>>().GetInstancedObject()->template GetObjectChecked<TBox<FReal, 3>>());
			}
			case ImplicitObjectType::IsInstanced | ImplicitObjectType::Capsule:
			{
				return Func(Geom.template GetObjectChecked< TImplicitObjectInstanced<FCapsule>>().GetInstancedObject()->template GetObjectChecked<FCapsule>());
			}
			case ImplicitObjectType::IsInstanced | ImplicitObjectType::Convex:
			{
				return Func(Geom.template GetObjectChecked< TImplicitObjectInstanced<FConvex>>().GetInstancedObject()->template GetObjectChecked<FConvex>());
			}
			case ImplicitObjectType::Transformed:
			{
				const auto& ImplicitObjectTransformed = (Geom.template GetObjectChecked<TImplicitObjectTransformed<FReal, 3>>());
				return CastHelper(*ImplicitObjectTransformed.GetTransformedObject(), Func);
			}
			default: 
				check(false);
			}
			return Func(Geom.template GetObjectChecked<FSphere>());	//needed for return type
		}

		// Call the lambda with concrete shape type. Unwraps shapes contained in Instanced (e.g., Instanced-Sphere will be called with Sphere. Note that this will effectively discard any instance properties like the margin)
		template <typename Lambda>
		FORCEINLINE_DEBUGGABLE decltype(auto) CastHelper(const FImplicitObject& Geom, const FRigidTransform3& TM, const Lambda& Func)
		{
			const EImplicitObjectType Type = Geom.GetType();
			switch (Type)
			{
			case ImplicitObjectType::Sphere: return Func(Geom.template GetObjectChecked<FSphere>(), TM);
			case ImplicitObjectType::Box: return Func(Geom.template GetObjectChecked<TBox<FReal, 3>>(), TM);
			case ImplicitObjectType::Capsule: return Func(Geom.template GetObjectChecked<FCapsule>(), TM);
			case ImplicitObjectType::Convex: return Func(Geom.template GetObjectChecked<FConvex>(), TM);
			case ImplicitObjectType::IsScaled | ImplicitObjectType::Sphere: return Func(Geom.template GetObjectChecked<TImplicitObjectScaled<FSphere>>(), TM);
			case ImplicitObjectType::IsScaled | ImplicitObjectType::Box: return Func(Geom.template GetObjectChecked< TImplicitObjectScaled<TBox<FReal, 3>>>(), TM);
			case ImplicitObjectType::IsScaled | ImplicitObjectType::Capsule: return Func(Geom.template GetObjectChecked< TImplicitObjectScaled<FCapsule>>(), TM);
			case ImplicitObjectType::IsScaled | ImplicitObjectType::Convex: return Func(Geom.template GetObjectChecked< TImplicitObjectScaled<FConvex>>(), TM);
			case ImplicitObjectType::IsInstanced | ImplicitObjectType::Sphere:
			{
				return Func(Geom.template GetObjectChecked< TImplicitObjectInstanced<FSphere>>().GetInstancedObject()->template GetObjectChecked<FSphere>(), TM);
			}
			case ImplicitObjectType::IsInstanced | ImplicitObjectType::Box:
			{
				return Func(Geom.template GetObjectChecked< TImplicitObjectInstanced<TBox<FReal, 3>>>().GetInstancedObject()->template GetObjectChecked<TBox<FReal, 3>>(), TM);
			}
			case ImplicitObjectType::IsInstanced | ImplicitObjectType::Capsule:
			{
				return Func(Geom.template GetObjectChecked< TImplicitObjectInstanced<FCapsule>>().GetInstancedObject()->template GetObjectChecked<FCapsule>(), TM);
			}
			case ImplicitObjectType::IsInstanced | ImplicitObjectType::Convex:
			{
				return Func(Geom.template GetObjectChecked< TImplicitObjectInstanced<FConvex>>().GetInstancedObject()->template GetObjectChecked<FConvex>(), TM);
			}
			case ImplicitObjectType::Transformed:
			{
				const auto& ImplicitObjectTransformed = (Geom.template GetObjectChecked<TImplicitObjectTransformed<FReal, 3>>());
				return CastHelper(*ImplicitObjectTransformed.GetTransformedObject(), ImplicitObjectTransformed.GetTransform() * TM, Func);
			}

			default: check(false);
			}
			return Func(Geom.template GetObjectChecked<FSphere>(), TM);	//needed for return type
		}

		// Call the lambda with concrete shape type. This version does NOT unwrap shapes contained in Instanced or Scaled.
		template <typename Lambda>
		FORCEINLINE_DEBUGGABLE decltype(auto) CastHelperNoUnwrap(const FImplicitObject& Geom, const FRigidTransform3& TM, const Lambda& Func)
		{
			const EImplicitObjectType Type = Geom.GetType();
			switch (Type)
			{
			case ImplicitObjectType::Sphere: return Func(Geom.template GetObjectChecked<FSphere>(), TM);
			case ImplicitObjectType::Box: return Func(Geom.template GetObjectChecked<TBox<FReal, 3>>(), TM);
			case ImplicitObjectType::Capsule: return Func(Geom.template GetObjectChecked<FCapsule>(), TM);
			case ImplicitObjectType::Convex: return Func(Geom.template GetObjectChecked<FConvex>(), TM);
			case ImplicitObjectType::IsScaled | ImplicitObjectType::Sphere: return Func(Geom.template GetObjectChecked<TImplicitObjectScaled<FSphere>>(), TM);
			case ImplicitObjectType::IsScaled | ImplicitObjectType::Box: return Func(Geom.template GetObjectChecked< TImplicitObjectScaled<TBox<FReal, 3>>>(), TM);
			case ImplicitObjectType::IsScaled | ImplicitObjectType::Capsule: return Func(Geom.template GetObjectChecked< TImplicitObjectScaled<FCapsule>>(), TM);
			case ImplicitObjectType::IsScaled | ImplicitObjectType::Convex: return Func(Geom.template GetObjectChecked< TImplicitObjectScaled<FConvex>>(), TM);
			case ImplicitObjectType::IsInstanced | ImplicitObjectType::Sphere: return Func(Geom.template GetObjectChecked< TImplicitObjectInstanced<FSphere>>(), TM);
			case ImplicitObjectType::IsInstanced | ImplicitObjectType::Box: return Func(Geom.template GetObjectChecked< TImplicitObjectInstanced<TBox<FReal, 3>>>(), TM);
			case ImplicitObjectType::IsInstanced | ImplicitObjectType::Capsule: return Func(Geom.template GetObjectChecked< TImplicitObjectInstanced<FCapsule>>(), TM);
			case ImplicitObjectType::IsInstanced | ImplicitObjectType::Convex: return Func(Geom.template GetObjectChecked< TImplicitObjectInstanced<FConvex>>(), TM);
			case ImplicitObjectType::Transformed:
			{
				const auto& ImplicitObjectTransformed = (Geom.template GetObjectChecked<TImplicitObjectTransformed<FReal, 3>>());
				return CastHelper(*ImplicitObjectTransformed.GetTransformedObject(), ImplicitObjectTransformed.GetTransform() * TM, Func);
			}

			default: check(false);
			}
			return Func(Geom.template GetObjectChecked<FSphere>(), TM);	//needed for return type
		}

		template <typename Lambda>
		FORCEINLINE_DEBUGGABLE decltype(auto) CastHelperNoUnwrap(const FImplicitObject& Geom, const FRigidTransform3& TM0, const FRigidTransform3& TM1, const Lambda& Func)
		{
			const EImplicitObjectType Type = Geom.GetType();
			switch (Type)
			{
			case ImplicitObjectType::Sphere: return Func(Geom.template GetObjectChecked<FSphere>(), TM0, TM1);
			case ImplicitObjectType::Box: return Func(Geom.template GetObjectChecked<TBox<FReal, 3>>(), TM0, TM1);
			case ImplicitObjectType::Capsule: return Func(Geom.template GetObjectChecked<FCapsule>(), TM0, TM1);
			case ImplicitObjectType::Convex: return Func(Geom.template GetObjectChecked<FConvex>(), TM0, TM1);
			case ImplicitObjectType::IsScaled | ImplicitObjectType::Sphere: return Func(Geom.template GetObjectChecked<TImplicitObjectScaled<FSphere>>(), TM0, TM1);
			case ImplicitObjectType::IsScaled | ImplicitObjectType::Box: return Func(Geom.template GetObjectChecked< TImplicitObjectScaled<TBox<FReal, 3>>>(), TM0, TM1);
			case ImplicitObjectType::IsScaled | ImplicitObjectType::Capsule: return Func(Geom.template GetObjectChecked< TImplicitObjectScaled<FCapsule>>(), TM0, TM1);
			case ImplicitObjectType::IsScaled | ImplicitObjectType::Convex: return Func(Geom.template GetObjectChecked< TImplicitObjectScaled<FConvex>>(), TM0, TM1);
			case ImplicitObjectType::IsInstanced | ImplicitObjectType::Sphere: return Func(Geom.template GetObjectChecked< TImplicitObjectInstanced<FSphere>>(), TM0, TM1);
			case ImplicitObjectType::IsInstanced | ImplicitObjectType::Box: return Func(Geom.template GetObjectChecked< TImplicitObjectInstanced<TBox<FReal, 3>>>(), TM0, TM1);
			case ImplicitObjectType::IsInstanced | ImplicitObjectType::Capsule: return Func(Geom.template GetObjectChecked< TImplicitObjectInstanced<FCapsule>>(), TM0, TM1);
			case ImplicitObjectType::IsInstanced | ImplicitObjectType::Convex: return Func(Geom.template GetObjectChecked< TImplicitObjectInstanced<FConvex>>(), TM0, TM1);
			case ImplicitObjectType::Transformed:
			{
				const auto& ImplicitObjectTransformed = (Geom.template GetObjectChecked<TImplicitObjectTransformed<FReal, 3>>());
				return CastHelperNoUnwrap(*ImplicitObjectTransformed.GetTransformedObject(), ImplicitObjectTransformed.GetTransform() * TM0, ImplicitObjectTransformed.GetTransform() * TM1, Func);
			}

			default: check(false);
			}
			return Func(Geom.template GetObjectChecked<FSphere>(), TM0, TM1);	//needed for return type
		}

		inline
		const FImplicitObject* ImplicitChildHelper(const FImplicitObject* ImplicitObject)
		{
			EImplicitObjectType ImplicitType = ImplicitObject->GetType();
			if (ImplicitType == TImplicitObjectTransformed<FReal, 3>::StaticType())
			{
				return ImplicitObject->template GetObject<const TImplicitObjectTransformed<FReal, 3>>()->GetTransformedObject();
			}

			else if (ImplicitType == TImplicitObjectScaled<FConvex>::StaticType())
			{
				return ImplicitObject->template GetObject<const TImplicitObjectScaled<FConvex>>()->GetUnscaledObject();
			}
			else if (ImplicitType == TImplicitObjectScaled<TBox<FReal, 3>>::StaticType())
			{
				return ImplicitObject->template GetObject<const TImplicitObjectScaled<TBox<FReal, 3>>>()->GetUnscaledObject();
			}
			else if (ImplicitType == TImplicitObjectScaled<FCapsule>::StaticType())
			{
				return ImplicitObject->template GetObject<const TImplicitObjectScaled<FCapsule>>()->GetUnscaledObject();
			}
			else if (ImplicitType == TImplicitObjectScaled<FSphere>::StaticType())
			{
				return ImplicitObject->template GetObject<const TImplicitObjectScaled<FSphere>>()->GetUnscaledObject();
			}
			else if (ImplicitType == TImplicitObjectScaled<FTriangleMeshImplicitObject>::StaticType())
			{
				return ImplicitObject->template GetObject<const TImplicitObjectScaled<FTriangleMeshImplicitObject>>()->GetUnscaledObject();
			}

			else if (ImplicitType == TImplicitObjectInstanced<FConvex>::StaticType())
			{
				return ImplicitObject->template GetObject<const TImplicitObjectInstanced<FConvex>>()->GetInstancedObject();
			}
			else if (ImplicitType == TImplicitObjectInstanced<TBox<FReal, 3>>::StaticType())
			{
				return ImplicitObject->template GetObject<const TImplicitObjectInstanced<TBox<FReal, 3>>>()->GetInstancedObject();
			}
			else if (ImplicitType == TImplicitObjectInstanced<FCapsule>::StaticType())
			{
				return ImplicitObject->template GetObject<const TImplicitObjectInstanced<FCapsule>>()->GetInstancedObject();
			}
			else if (ImplicitType == TImplicitObjectInstanced<FSphere>::StaticType())
			{
				return ImplicitObject->template GetObject<const TImplicitObjectInstanced<FSphere>>()->GetInstancedObject();
			}
			else if (ImplicitType == TImplicitObjectInstanced<FTriangleMeshImplicitObject>::StaticType())
			{
				return ImplicitObject->template GetObject<const TImplicitObjectInstanced<FTriangleMeshImplicitObject>>()->GetInstancedObject();
			}
			return nullptr;
		}

		/**
		 * @brief Given an ImplicitObject known to be of type T or a wrapper around T, call the Lambda with the concrete type
		 * which will be const T*, const TImplicitObjectInstanced<T>*, or const TImplicitObjectScaled<T>*
		*/
		template <typename T, typename Lambda>
		FORCEINLINE_DEBUGGABLE decltype(auto) CastWrapped(const FImplicitObject& A, const Lambda& Func)
		{
			if (const TImplicitObjectScaled<T>* ScaledImplicit = A.template GetObject<TImplicitObjectScaled<T>>())
			{
				return Func(ScaledImplicit);
			}
			else if (const TImplicitObjectInstanced<T>* InstancedImplicit = A.template GetObject<TImplicitObjectInstanced<T>>())
			{
				return Func(InstancedImplicit);
			}
			else if(const T* RawImplicit = A.template GetObject<T>())
			{
				return Func(RawImplicit);
			}
			else
			{
				return Func((T*)nullptr);
			}
		}

		template <typename Lambda, bool bRootObject>
		FORCEINLINE_DEBUGGABLE void VisitConcreteObjectsImpl(const FImplicitObject& Geom, const Lambda& Func, int32 RootObjectIndex)
		{
			if (const FImplicitObjectUnion* Union = Geom.AsA<FImplicitObjectUnion>())
			{
				for (const FImplicitObjectPtr& ObjectPtr : Union->GetObjects())
				{
					if (const FImplicitObject* Object = ObjectPtr.GetReference())
					{
						VisitConcreteObjectsImpl<Lambda, false>(*Object, Func, RootObjectIndex);
					}

					if constexpr (bRootObject)
					{
						++RootObjectIndex;
					}
				}
			}
			else
			{
				CastHelper(Geom, [&Func, RootObjectIndex](const auto& Concrete)
				{
					Func(Concrete, RootObjectIndex);
				});
			}
		}

		/**
		 * @brief Similar to FImplicitObject::VisitLeafObjects, but provides only the root object index
		 * (ie, the index into a particle's ShapeInstance array) and the concrete type of the leaf object.
		 * This is useful to avoid requiring users to write lambdas inside lambdas or do manual type
		 * checking if all the caller wants to do is perform some operation on each of the underlying geometries.
		 */
		template <typename Lambda>
		FORCEINLINE_DEBUGGABLE void VisitConcreteObjects(const FImplicitObject& Geom, const Lambda& Func)
		{
			VisitConcreteObjectsImpl<Lambda, true>(Geom, Func, 0);
		}

	} // namespace Utilities

} // namespace Chaos
