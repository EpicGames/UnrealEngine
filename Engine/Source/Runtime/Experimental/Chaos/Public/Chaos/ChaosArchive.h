// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Real.h"
#include "CoreMinimal.h"
#include "Serializable.h"
#include "ShapeInstanceFwd.h"
#include "Serialization/ArchiveProxy.h"
#include "UObject/DestructionObjectVersion.h"
#include "Templates/Models.h"

class FChaosVisualDebuggerTrace;

namespace Chaos
{

#if CHAOS_MEMORY_TRACKING
struct FChaosArchiveSection
{
	FName Name;
	int64 MemoryLocation;
	int64 ChildMemory;
	bool bAbsorbChildren;
};

struct FChaosArchiveSectionData
{
	FChaosArchiveSectionData() : Count(0), SizeInclusive(0), SizeExclusive(0) { }
	int32 Count;
	int64 SizeInclusive;
	int64 SizeExclusive;
};

class FChaosArchiveMemoryTrackingContext
{
public:
	FChaosArchiveMemoryTrackingContext()
		: ChildAbsorbers(0)
		, UntrackedSerializations(0) { }

	~FChaosArchiveMemoryTrackingContext()
	{
		check(SectionStack.Num() == 0);
	}

	TMap<FName, FChaosArchiveSectionData> SectionMap;
	TArray<FChaosArchiveSection> SectionStack;

	CHAOS_API void PushSection(const FName& SectionName, const int64 MemoryLocation, const bool bAbsorbChildren);

	CHAOS_API void PopSection(const int64 MemoryLocation);

	CHAOS_API void BeginSerialize(const int64 MemoryLocation);

	CHAOS_API void EndSerialize(const int64 MemoryLocation);

private:

	static const FName UntrackedName;

	int32 ChildAbsorbers;

	int32 UntrackedSerializations;
};

class FChaosArchive;

#endif



class FChaosArchiveContext
#if CHAOS_MEMORY_TRACKING
	: public FChaosArchiveMemoryTrackingContext
#endif
{
public:
	TArray<void*> TagToObject;
	TMap<void*, int32> ObjToTag;
	TSet<void*> PendingAdds;
	int32 TagCount;


	CHAOS_API FChaosArchiveContext();

	CHAOS_API ~FChaosArchiveContext();

	template <typename T, ESPMode Mode>
	TSharedPtr<T, Mode>& ToSharedPointerHelper(TSerializablePtr<T>& Obj)
	{
		T* RawPtr = const_cast<T*>(Obj.Get());
		if (FSharedPtrHolder* BaseHolder = ObjToSharedPtrHolder.FindRef(RawPtr))
		{
			auto ConcreteHolder = static_cast<TSharedPtrHolder<T, Mode>*>(BaseHolder);
			return ConcreteHolder->SP;
		}
		else
		{
			auto NewHolder = new TSharedPtrHolder<T, Mode>(RawPtr);
			TSharedPtr<T, Mode>& NewSP = NewHolder->SP;
			ObjToSharedPtrHolder.Add((void*)RawPtr, NewHolder);
			return NewSP;
		}
	}

	template <typename T>
	TRefCountPtr<T>& ToRefCountPointerHelper(TSerializablePtr<T>& Obj)
	{
		T* RawPtr = const_cast<T*>(Obj.Get());
		if (FRefCountPtrHolder* BaseHolder = ObjToRefCountPtrHolder.FindRef(RawPtr))
		{
			auto ConcreteHolder = static_cast<TRefCountPtrHolder<T>*>(BaseHolder);
			return ConcreteHolder->RCP;
		}
		else
		{
			auto NewHolder = new TRefCountPtrHolder<T>(RawPtr);
			TRefCountPtr<T>& NewRCP = NewHolder->RCP;
			ObjToRefCountPtrHolder.Add((void*)RawPtr, NewHolder);
			return NewRCP;
		}
	}

	int32 GetObjectTag(const void* ObjectPtr) const
	{
		if (const int32* SerializedObjectPtrTag = ObjToTag.Find(ObjectPtr))
		{
			return *SerializedObjectPtrTag;
		}
		return INDEX_NONE;
	}

private:
	class FSharedPtrHolder
	{
	public:
		FSharedPtrHolder() = default;
		virtual ~FSharedPtrHolder() = default;
	};

	template <typename T, ESPMode Mode>
	class TSharedPtrHolder : public FSharedPtrHolder
	{
	public:
		TSharedPtrHolder(T* Obj) : SP(Obj) {}
		TSharedPtr<T, Mode> SP;
	};
	
	class FRefCountPtrHolder
	{
	public:
		FRefCountPtrHolder() = default;
		virtual ~FRefCountPtrHolder() = default;
	};

	template <typename T>
	class TRefCountPtrHolder : public FRefCountPtrHolder
	{
	public:
		TRefCountPtrHolder(T* Obj) : RCP(Obj) {}
		TRefCountPtr<T> RCP;
	};

	TMap<void*, FSharedPtrHolder*> ObjToSharedPtrHolder;
	TMap<void*, FRefCountPtrHolder*> ObjToRefCountPtrHolder;
};

class FChaosArchive : public FArchiveProxy
{
public:
	FChaosArchive(FArchive& ArIn)
		: FArchiveProxy(ArIn)
		, Context(MakeShared<FChaosArchiveContext>())
	{
	}

protected:
	FChaosArchive(FArchive& ArIn, const TSharedRef<FChaosArchiveContext>& InExternalSharedContext)
	: FArchiveProxy(ArIn)
	, Context(InExternalSharedContext)
	{
	}
	
public:

	template <typename T>
	void SerializePtr(TSerializablePtr<T>& Obj)
	{
		bool bExists = Obj.Get() != nullptr;
		InnerArchive << bExists;
		if (!bExists)
		{
			Obj.Reset();
			return;
		}

		if (InnerArchive.IsLoading())
		{
			int32 Tag;
			InnerArchive << Tag;

			if (Tag < 0)
			{
				InnerArchive.SetCriticalError();
				return;
			}

			const int32 SlotsNeeded = Tag + 1 - Context->TagToObject.Num();
			if (SlotsNeeded > 0)
			{
				Context->TagToObject.AddZeroed(SlotsNeeded);
			}

			if (!Context->TagToObject.IsValidIndex(Tag))
			{
				InnerArchive.SetCriticalError();
				return;
			}

			if (Context->TagToObject[Tag])
			{
				Obj.SetFromRawLowLevel((const T*)(Context->TagToObject[Tag]));
			}
			else
			{
				StaticSerialize(Obj);
				Context->TagToObject[Tag] = (void*)Obj.Get();
				Context->ObjToTag.Add((void*)Obj.Get(), Tag);
			}
		}
		else if (InnerArchive.IsSaving() || InnerArchive.IsCountingMemory())
		{
			void* ObjRaw = (void*)Obj.Get();
			check(Context->PendingAdds.Contains(ObjRaw) == false);	//catch dependency cycles. Not supported
			if (Context->ObjToTag.Contains(ObjRaw))
			{
				InnerArchive << Context->ObjToTag[ObjRaw];
			}
			else
			{
				Context->PendingAdds.Add(ObjRaw);

				int32 Tag = Context->TagCount++;
				Context->ObjToTag.Add(ObjRaw, Tag);

				InnerArchive << Tag;
				StaticSerialize(Obj);

				Context->PendingAdds.Remove(ObjRaw);
			}
		}
	}

	template <typename T>
	void SerializePtr(TUniquePtr<T>& Obj)
	{
		InnerArchive.UsingCustomVersion(FDestructionObjectVersion::GUID);
		if (InnerArchive.CustomVer(FDestructionObjectVersion::GUID) < FDestructionObjectVersion::ChaosArchiveAdded)
		{
			SerializeLegacy(Obj);
		}
		else
		{
			TSerializablePtr<T> Copy(Obj);
			SerializePtr(Copy);

			if (InnerArchive.IsLoading())
			{
				//check(!Obj);	sometimes we have a default object. Maybe we shouldn't?
				Obj.Reset(const_cast<T*>(Copy.Get()));
			}
		}
	}

	template <typename T>
	void SerializePtr(TRefCountPtr<T>& Obj)
	{
		TSerializablePtr<T> Copy = MakeSerializable(Obj);
		SerializePtr(Copy);
		if (IsLoading())
		{
			Obj = Context->ToRefCountPointerHelper<T>(Copy);
		}
	}

	template <typename T>
	void SerializeConstPtr(TRefCountPtr<const T>& Obj)
	{
		TSerializablePtr<T> Copy = MakeSerializable(Obj);
		SerializePtr(Copy);
		if (IsLoading())
		{
			Obj = Context->ToRefCountPointerHelper<T>(Copy);
		}
	}

	template <typename T, ESPMode Mode>
	void SerializePtr(TSharedPtr<T, Mode>& Obj)
	{
		TSerializablePtr<T> Copy = MakeSerializable(Obj);
		SerializePtr(Copy);
		if (IsLoading())
		{
			Obj = Context->ToSharedPointerHelper<T,Mode>(Copy);
		}
	}

	template <typename T, ESPMode Mode>
	void SerializeConstPtr(TSharedPtr<const T, Mode>& Obj)
	{
		TSerializablePtr<T> Copy = MakeSerializable(Obj);
		SerializePtr(Copy);
		if (IsLoading())
		{
			Obj = Context->ToSharedPointerHelper<T, Mode>(Copy);
		}
	}

#if CHAOS_MEMORY_TRACKING
	virtual void Serialize(void* V, int64 Length) override
	{
		Context->BeginSerialize(Tell());
		FArchiveProxy::Serialize(V, Length);
		Context->EndSerialize(Tell());
	}
#endif

	UE_DEPRECATED(5.8, "This method is no longer supported and there is no replacement")
	void SetContext(TUniquePtr<FChaosArchiveContext>&& InContext)
	{
	}

	UE_DEPRECATED(5.8, "This method is no longer supported and there is no replacement")
	TUniquePtr<FChaosArchiveContext> StealContext()
	{
		return MakeUnique<FChaosArchiveContext>();
	}

private:

	template <typename T>
	void SerializeLegacy(TUniquePtr<T>& Obj)
	{
		check(false);
	}
	CHAOS_API void SerializeLegacy(TUniquePtr<FImplicitObject>& Obj);

	template <typename T>
	void StaticSerialize(TSerializablePtr<T>& Serializable)
	{
		T* RawPtr = const_cast<T*>(Serializable.Get());
		FChaosArchive& Ar = *this;
		if (auto CreatedObj = T::SerializationFactory(Ar, RawPtr))
		{
			check(Ar.IsLoading());	//SerializationFactory should only create objects on load
			RawPtr = static_cast<T*>(CreatedObj);
			Serializable.SetFromRawLowLevel(RawPtr);
		}
		else
		{
			check(!Ar.IsLoading())	//SerializationFactory must construct new object on load
		}
		
		RawPtr->Serialize(Ar);
	}

	TSharedRef<FChaosArchiveContext> Context;

#if CHAOS_MEMORY_TRACKING
	friend class FChaosArchiveScopedMemory;
#endif
};

class FChaosArchiveScopedMemory
{
public:
	FChaosArchiveScopedMemory(FChaosArchive& ArIn, const FName& SectionName, const bool bAbsorbChildren = true)
#if CHAOS_MEMORY_TRACKING
	: Ar(ArIn)
	{
		Ar.Context->PushSection(SectionName, Ar.Tell(), bAbsorbChildren);
	}

	~FChaosArchiveScopedMemory()
	{
		Ar.Context->PopSection(Ar.Tell());
	}
private:
	FChaosArchive& Ar;
#else
	{ }
#endif
};
	
template <typename T, typename TAllocator>
FChaosArchive& operator<<(FChaosArchive& Ar, TArray<T, TAllocator>& Array)
{
	Array.CountBytes(Ar);

	int32 ArrayNum = Array.Num();
	Ar << ArrayNum;
	Array.Reserve(ArrayNum);
	Array.SetNum(ArrayNum);

	for (int32 Idx = 0; Idx < ArrayNum; ++Idx)
	{
		Ar << Array[Idx];
	}

	return Ar;
}

FORCEINLINE FChaosArchive& operator<<(FChaosArchive& Ar, Chaos::FReal& Real)
{
	// we need to check if we are storing doubles or floats
	if (Ar.UEVer() >= EUnrealEngineObjectUE5Version::LARGE_WORLD_COORDINATES)
	{
		// normal umodified type path 
		operator<<((FArchive&)Ar, Real);
	}
	else
	{
		// in that case data is stored as float and we need to read it as such		
		//ensure(Ar.IsLoading()); // this case should normally only happening when reading 
		FRealSingle RealSingle = (FRealSingle)Real; 
		operator<<((FArchive&)Ar, RealSingle);
		if(Ar.IsLoading())
		{
			Real = (FReal)RealSingle;
		}
	}
	return Ar;
}

struct CSerializablePtr
{
	template<typename T>
	auto Requires(T* InType, FChaosArchive& InAr) -> decltype(T::SerializationFactory(InAr, (T*)nullptr));
};

template <typename T>
constexpr typename TEnableIf<TModels_V<CSerializablePtr, T>, bool>::Type IsSerializablePtr()
{
	return true;
}

template <typename T>
typename TEnableIf<IsSerializablePtr<T>(), FChaosArchive& > ::Type operator<<(FChaosArchive& Ar, TRefCountPtr<T>& Obj)
{
	Ar.SerializePtr(Obj);
	return Ar;
}

template <typename T>
typename TEnableIf<IsSerializablePtr<T>(), FChaosArchive& > ::Type operator<<(FChaosArchive& Ar, TUniquePtr<T>& Obj)
{
	Ar.SerializePtr(Obj);
	return Ar;
}

template <typename T, ESPMode Mode>
typename TEnableIf<IsSerializablePtr<T>(), FChaosArchive& > ::Type operator<<(FChaosArchive& Ar, TSharedPtr<T, Mode>& Obj)
{
	Ar.SerializePtr(Obj);
	return Ar;
}

template <typename T>
typename TEnableIf<IsSerializablePtr<T>(), FChaosArchive& > ::Type operator<<(FChaosArchive& Ar, TSerializablePtr<T>& Serializable)
{
	Ar.SerializePtr(Serializable);
	return Ar;
}

template <typename T>
typename TEnableIf<T::AlwaysSerializable, FChaosArchive& > ::Type operator<<(FChaosArchive& Ar, T*& Obj)
{
	Ar.SerializePtr(AsAlwaysSerializable(Obj));
	return Ar;
}

template <typename T, typename TAllocator>
typename TEnableIf<T::AlwaysSerializable, FChaosArchive& > ::Type operator<<(FChaosArchive& Ar, TArray<T*, TAllocator>& Array)
{
	Ar << AsAlwaysSerializableArray(Array);
	return Ar;
}
	
template <typename T, typename TAllocator>
typename TEnableIf<IsSerializablePtr<T>(), FChaosArchive& > ::Type operator<<(FChaosArchive& Ar, TArray<TRefCountPtr<T>, TAllocator>& Array)
{
	int32 ArrayNum = Array.Num();
	Ar << ArrayNum;
	Array.Reserve(ArrayNum);
	Array.SetNum(ArrayNum);

	for (int32 Idx = 0; Idx < ArrayNum; ++Idx)
	{
		Ar << Array[Idx];
	}

	return Ar;
}

template <typename T, typename TAllocator, typename TAllocator2>
typename TEnableIf<IsSerializablePtr<T>(), FChaosArchive& > ::Type operator<<(FChaosArchive& Ar, TArray<TArray<TRefCountPtr<T>, TAllocator>,TAllocator2>& Array)
{
	int32 ArrayNum = Array.Num();
	Ar << ArrayNum;
	Array.Reserve(ArrayNum);
	Array.SetNum(ArrayNum);

	for (int32 Idx = 0; Idx < ArrayNum; ++Idx)
	{
		Ar << Array[Idx];
	}

	return Ar;
}
	
template <typename T, typename TAllocator>
typename TEnableIf<IsSerializablePtr<T>(), FChaosArchive& > ::Type operator<<(FChaosArchive& Ar, TArray<TUniquePtr<T>, TAllocator>& Array)
{
	int32 ArrayNum = Array.Num();
	Ar << ArrayNum;
	Array.Reserve(ArrayNum);
	Array.SetNum(ArrayNum);

	for (int32 Idx = 0; Idx < ArrayNum; ++Idx)
	{
		Ar << Array[Idx];
	}

	return Ar;
}
	
template <typename T, typename TAllocator, typename TAllocator2>
typename TEnableIf<IsSerializablePtr<T>(), FChaosArchive& > ::Type operator<<(FChaosArchive& Ar, TArray<TArray<TUniquePtr<T>, TAllocator>,TAllocator2>& Array)
{
	int32 ArrayNum = Array.Num();
	Ar << ArrayNum;
	Array.Reserve(ArrayNum);
	Array.SetNum(ArrayNum);

	for (int32 Idx = 0; Idx < ArrayNum; ++Idx)
	{
		Ar << Array[Idx];
	}

	return Ar;
}

template <typename T, typename TAllocator, ESPMode Mode>
typename TEnableIf<IsSerializablePtr<T>(), FChaosArchive& > ::Type operator<<(FChaosArchive& Ar, TArray<TSharedPtr<T, Mode>, TAllocator>& Array)
{
	int32 ArrayNum = Array.Num();
	Ar << ArrayNum;
	Array.Reserve(ArrayNum);
	Array.SetNum(ArrayNum);

	for (int32 Idx = 0; Idx < ArrayNum; ++Idx)
	{
		Ar << Array[Idx];
	}

	return Ar;
}

template <typename T, typename TAllocator, typename TAllocator2, ESPMode Mode>
typename TEnableIf<IsSerializablePtr<T>(), FChaosArchive& > ::Type operator<<(FChaosArchive& Ar, TArray<TArray<TSharedPtr<T, Mode>, TAllocator>, TAllocator2>& Array)
{
	int32 ArrayNum = Array.Num();
	Ar << ArrayNum;
	Array.Reserve(ArrayNum);
	Array.SetNum(ArrayNum);

	for (int32 Idx = 0; Idx < ArrayNum; ++Idx)
	{
		Ar << Array[Idx];
	}

	return Ar;
}

template <typename T, typename TAllocator>
typename TEnableIf<IsSerializablePtr<T>(), FChaosArchive& > ::Type operator<<(FChaosArchive& Ar, TArray<TSerializablePtr<T>, TAllocator>& Array)
{
	int32 ArrayNum = Array.Num();
	Ar << ArrayNum;
	Array.SetNum(ArrayNum);

	for (int32 Idx = 0; Idx < ArrayNum; ++Idx)
	{
		Ar << Array[Idx];
	}

	return Ar;
}

template <typename T, typename TAllocator, typename TAllocator2>
typename TEnableIf<IsSerializablePtr<T>(), FChaosArchive& > ::Type operator<<(FChaosArchive& Ar, TArray<TArray<TSerializablePtr<T>, TAllocator>, TAllocator2>& Array)
{
	int32 ArrayNum = Array.Num();
	Ar << ArrayNum;
	Array.Reserve(ArrayNum);
	Array.SetNum(ArrayNum);

	for (int32 Idx = 0; Idx < ArrayNum; ++Idx)
	{
		Ar << Array[Idx];
	}

	return Ar;
}

}
