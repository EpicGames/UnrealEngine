// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Declares.h"
#include "Chaos/ChaosDebugDrawDeclares.h"
#include "Chaos/KinematicGeometryParticles.h"
#include "Chaos/PBDRigidParticles.h"
#include "Chaos/Vector.h"

namespace Chaos
{
	namespace Private
	{
		class FCollisionConstraintAllocator;
	}

	class FAccelerationStructureHandle;
	struct FCCDConstraint;
	template <typename PayloadType, typename T, int d> class ISpatialAcceleration;
	class FPBDSuspensionConstraints;
	class FRigidClustering;
	class FShapeOrShapesArray;

	namespace DebugDraw
	{
#if CHAOS_DEBUG_DRAW

		struct FChaosDebugDrawColorsByState
		{
			CHAOS_API FChaosDebugDrawColorsByState(
				FColor InDynamicColor,
				FColor InSleepingColor,
				FColor InKinematicColor,
				FColor InStaticColor,
				FColor InDebrisColor = FColor::Green
			);

			FColor DynamicColor;
			FColor SleepingColor;
			FColor KinematicColor;
			FColor StaticColor;
			FColor DebrisColor;

			CHAOS_API FColor GetColorFromState(EObjectStateType State, bool bIsDebris = false) const;
		};

		struct FChaosDebugDrawColorsByShapeType
		{
			CHAOS_API FChaosDebugDrawColorsByShapeType(
				FColor InSimpleTypeColor,
				FColor InConvexColor,
				FColor InHeightFieldColor,
				FColor InTriangleMeshColor,
				FColor InLevelSetColor
			);

			//Note: add entries in order to avoid serialization issues (but before IsInstanced)
			FColor SimpleTypeColor; // Sphere, Plane, Cube. Capsule, Cylinder, tapered shapes
			FColor ConvexColor;
			FColor HeightFieldColor;
			FColor TriangleMeshColor;
			FColor LevelSetColor;

			CHAOS_API FColor GetColorFromShapeType(EImplicitObjectType ShapeType) const;
		};

		struct FChaosDebugDrawSettings
		{
		public:
			FChaosDebugDrawSettings(
				FRealSingle InArrowSize,
				FRealSingle InBodyAxisLen,
				FRealSingle InContactLen,
				FRealSingle InContactWidth,
				FRealSingle InContactInfoWidth,
				FRealSingle InContactOwnerWidth,
				FRealSingle InConstraintAxisLen,
				FRealSingle InJointComSize,
				FRealSingle InLineThickness,
				FRealSingle InDrawScale,
				FRealSingle InFontHeight,
				FRealSingle InFontScale,
				FRealSingle InShapeThicknesScale,
				FRealSingle InPointSize,
				FRealSingle InVelScale,
				FRealSingle InAngVelScale,
				FRealSingle InImpulseScale,
				FRealSingle InPushOutScale,
				FRealSingle InInertiaScale,
				int32 InDrawPriority,
				bool bInShowSimpleCollision,
				bool bInShowComplexCollision,
				bool bInShowLevelSetCollision,
				const FChaosDebugDrawColorsByState& InShapesColorsPerState,
				const FChaosDebugDrawColorsByShapeType& InShapesColorsPerShapeType,
				const FChaosDebugDrawColorsByState& InBoundsColorsPerState,
				const FChaosDebugDrawColorsByShapeType& InBoundsColorsPerShapeType
				)
				: ArrowSize(InArrowSize)
				, BodyAxisLen(InBodyAxisLen)
				, ContactLen(InContactLen)
				, ContactWidth(InContactWidth)
				, ContactInfoWidth(InContactInfoWidth)
				, ContactOwnerWidth(InContactOwnerWidth)
				, ConstraintAxisLen(InConstraintAxisLen)
				, JointComSize(InJointComSize)
				, LineThickness(InLineThickness)
				, DrawScale(InDrawScale)
				, FontHeight(InFontHeight)
				, FontScale(InFontScale)
				, ShapeThicknesScale(InShapeThicknesScale)
				, PointSize(InPointSize)
				, VelScale(InVelScale)
				, AngVelScale(InAngVelScale)
				, ImpulseScale(InImpulseScale)
				, PushOutScale(InPushOutScale)
				, InertiaScale(InInertiaScale)
				, DrawPriority(InDrawPriority)
				, bShowSimpleCollision(bInShowSimpleCollision)
				, bShowComplexCollision(bInShowComplexCollision)
				, bShowLevelSetCollision(bInShowLevelSetCollision)
				, ShapesColorsPerState(InShapesColorsPerState)
				, ShapesColorsPerShapeType(InShapesColorsPerShapeType)
				, BoundsColorsPerState(InBoundsColorsPerState)
				, BoundsColorsPerShapeType(InBoundsColorsPerShapeType)
			{
			}

			FRealSingle ArrowSize;
			FRealSingle BodyAxisLen;
			FRealSingle ContactLen;
			FRealSingle ContactWidth;
			FRealSingle ContactInfoWidth;
			FRealSingle ContactOwnerWidth;
			FRealSingle ConstraintAxisLen;
			FRealSingle JointComSize;
			FRealSingle LineThickness;
			FRealSingle DrawScale;
			FRealSingle FontHeight;
			FRealSingle FontScale;
			FRealSingle ShapeThicknesScale;
			FRealSingle PointSize;
			FRealSingle VelScale;
			FRealSingle AngVelScale;
			FRealSingle ImpulseScale;
			FRealSingle PushOutScale;
			FRealSingle InertiaScale;
			FRealSingle DrawDuration;
			int32 DrawPriority;
			bool bShowSimpleCollision;
			bool bShowComplexCollision;
			bool bShowLevelSetCollision;
			FChaosDebugDrawColorsByState ShapesColorsPerState;
			FChaosDebugDrawColorsByShapeType ShapesColorsPerShapeType;
			FChaosDebugDrawColorsByState BoundsColorsPerState;
			FChaosDebugDrawColorsByShapeType BoundsColorsPerShapeType;
		};

		// A bitmask of features to show when drawing joints
		class FChaosDebugDrawJointFeatures
		{
		public:
			FChaosDebugDrawJointFeatures()
				: bCoMConnector(false)
				, bActorConnector(false)
				, bStretch(false)
				, bAxes(false)
				, bLevel(false)
				, bIndex(false)
				, bColor(false)
				, bIsland(false)
			{}

			static FChaosDebugDrawJointFeatures MakeEmpty()
			{
				return FChaosDebugDrawJointFeatures();
			}

			static FChaosDebugDrawJointFeatures MakeDefault()
			{
				FChaosDebugDrawJointFeatures Features = FChaosDebugDrawJointFeatures();
				Features.bActorConnector = true;
				Features.bStretch = true;
				return Features;
			}

			bool bCoMConnector;
			bool bActorConnector;
			bool bStretch;
			bool bAxes;
			bool bLevel;
			bool bIndex;
			bool bColor;
			bool bIsland;
		};

		CHAOS_API const FChaosDebugDrawColorsByState& GetDefaultShapesColorsByState();
		CHAOS_API const FChaosDebugDrawColorsByState& GetDefaultBoundsColorsByState();

		CHAOS_API const FChaosDebugDrawColorsByShapeType& GetDefaultShapesColorsByShapeType();
		CHAOS_API const FChaosDebugDrawColorsByShapeType& GetDefaultBoundsColorsByShapeType();

		CHAOS_API void DrawParticleShapes(const FRigidTransform3& SpaceTransform, const TParticleView<FGeometryParticles>& ParticlesView,  FReal ColorScale, const FChaosDebugDrawSettings* Settings = nullptr);
		CHAOS_API void DrawParticleShapes(const FRigidTransform3& SpaceTransform, const TParticleView<FKinematicGeometryParticles>& ParticlesView,  FReal ColorScale, const FChaosDebugDrawSettings* Settings = nullptr);
		CHAOS_API void DrawParticleShapes(const FRigidTransform3& SpaceTransform, const TParticleView<FPBDRigidParticles>& ParticlesView,  FReal ColorScale, const FChaosDebugDrawSettings* Settings = nullptr);
		CHAOS_API void DrawParticleShapes(const FRigidTransform3& SpaceTransform, const FGeometryParticleHandle* Particle, const FRigidTransform3& ParticleTransform, const FColor& Color, const FChaosDebugDrawSettings* Settings = nullptr);
		CHAOS_API void DrawParticleShapes(const FRigidTransform3& SpaceTransform, const FGeometryParticleHandle* Particle, const FColor& Color, const FChaosDebugDrawSettings* Settings = nullptr);
		CHAOS_API void DrawParticleShapes(const FRigidTransform3& SpaceTransform, const FGeometryParticle* Particle, const FColor& Color, const FChaosDebugDrawSettings* Settings = nullptr);
		CHAOS_API void DrawParticleBVH(const FRigidTransform3& SpaceTransform, const FGeometryParticleHandle* Particle, const FColor& Color, const FChaosDebugDrawSettings* Settings = nullptr);
		CHAOS_API void DrawParticleBounds(const FRigidTransform3& SpaceTransform, const TParticleView<FGeometryParticles>& ParticlesView, const FReal Dt, const FChaosDebugDrawSettings* Settings = nullptr);
		CHAOS_API void DrawParticleBounds(const FRigidTransform3& SpaceTransform, const TParticleView<FKinematicGeometryParticles>& ParticlesView, const FReal Dt, const FChaosDebugDrawSettings* Settings = nullptr);
		CHAOS_API void DrawParticleBounds(const FRigidTransform3& SpaceTransform, const TParticleView<FPBDRigidParticles>& ParticlesView, const FReal Dt, const FChaosDebugDrawSettings* Settings = nullptr);
		CHAOS_API void DrawParticleTransforms(const FRigidTransform3& SpaceTransform, const TParticleView<FGeometryParticles>& ParticlesView, const FChaosDebugDrawSettings* Settings = nullptr);
		CHAOS_API void DrawParticleTransforms(const FRigidTransform3& SpaceTransform, const TParticleView<FKinematicGeometryParticles>& ParticlesView, const FChaosDebugDrawSettings* Settings = nullptr);
		CHAOS_API void DrawParticleTransforms(const FRigidTransform3& SpaceTransform, const TParticleView<FPBDRigidParticles>& ParticlesView, const FChaosDebugDrawSettings* Settings = nullptr);
		CHAOS_API void DrawParticleCollisions(const FRigidTransform3& SpaceTransform, const FGeometryParticleHandle* Particle, const FPBDCollisionConstraints& Collisions, const FChaosDebugDrawSettings* Settings = nullptr);
		CHAOS_API void DrawCollision(const FRigidTransform3& SpaceTransform, const FPBDCollisionConstraint& Collision, FRealSingle ColorScale, const FChaosDebugDrawSettings* Settings = nullptr);
		CHAOS_API void DrawCollisions(const FRigidTransform3& SpaceTransform, const FPBDCollisionConstraints& Collisions, FRealSingle ColorScale, const FChaosDebugDrawSettings* Settings = nullptr);
		CHAOS_API void DrawCollisions(const FRigidTransform3& SpaceTransform, const Private::FCollisionConstraintAllocator& CollisionAllocator, FRealSingle ColorScale, const FChaosDebugDrawSettings* Settings = nullptr);
		CHAOS_API void DrawParticleMass(const FRigidTransform3& SpaceTransform, const TParticleView<FKinematicGeometryParticles>& ParticlesView, const FChaosDebugDrawSettings* Settings = nullptr);
		CHAOS_API void DrawParticleMass(const FRigidTransform3& SpaceTransform, const TParticleView<FPBDRigidParticles>& ParticlesView, const FChaosDebugDrawSettings* Settings = nullptr);
		CHAOS_API void DrawParticleDensity(const FRigidTransform3& SpaceTransform, const TParticleView<FKinematicGeometryParticles>& ParticlesView, const FChaosDebugDrawSettings* Settings = nullptr);
		CHAOS_API void DrawParticleDensity(const FRigidTransform3& SpaceTransform, const TParticleView<FPBDRigidParticles>& ParticlesView, const FChaosDebugDrawSettings* Settings = nullptr);
		CHAOS_API void DrawJointConstraints(const FRigidTransform3& SpaceTransform, const TArray<FPBDJointConstraintHandle*>& ConstraintHandles, FRealSingle ColorScale, const FChaosDebugDrawJointFeatures& FeatureMask = FChaosDebugDrawJointFeatures::MakeDefault(), const FChaosDebugDrawSettings* Settings = nullptr);
		CHAOS_API void DrawJointConstraints(const FRigidTransform3& SpaceTransform, const FPBDJointConstraints& Constraints, FRealSingle ColorScale, const FChaosDebugDrawJointFeatures& FeatureMask = FChaosDebugDrawJointFeatures::MakeDefault(), const FChaosDebugDrawSettings* Settings = nullptr);
		CHAOS_API void DrawCharacterGroundConstraints(const FRigidTransform3& SpaceTransform, const FCharacterGroundConstraintContainer& Constraints, const FChaosDebugDrawSettings* Settings = nullptr);
		CHAOS_API void DrawShape(const FRigidTransform3& ShapeTransform, const FImplicitObject* Implicit, const FPerShapeData* Shape, const FColor& Color, const float Duration, const FChaosDebugDrawSettings* Settings = nullptr);
		CHAOS_API void DrawSimulationSpace(const FTransform& SpaceTransform, const FSimulationSpace& SimSpace, const FChaosDebugDrawSettings* Settings = nullptr);
		CHAOS_API void DrawConstraintGraph(const FRigidTransform3& ShapeTransform, const Private::FPBDIslandManager& Graph, const FChaosDebugDrawSettings* Settings = nullptr);
		CHAOS_API void DrawConnectionGraph(const FRigidClustering& Clustering, const FChaosDebugDrawSettings* Settings = nullptr);
		CHAOS_API void DrawCollidingShapes(const FRigidTransform3& SpaceTransform, const FPBDCollisionConstraints& Collisions, FRealSingle ColorScale, const FRealSingle Duration = 0.f, const FChaosDebugDrawSettings* Settings = nullptr);
		CHAOS_API void DrawCollidingShapes(const FRigidTransform3& SpaceTransform, const FPBDCollisionConstraint& Collision, FRealSingle ColorScale, const FRealSingle Duration = 0.f, const FChaosDebugDrawSettings* Settings = nullptr);
		CHAOS_API void DrawSpatialAccelerationStructure(const ISpatialAcceleration<FAccelerationStructureHandle, FReal, 3>& SpatialAccelerationStructure, const FChaosDebugDrawSettings* Settings = nullptr);
		CHAOS_API void DrawSuspensionConstraints(const FRigidTransform3& SpaceTransform, const FPBDSuspensionConstraints& Constraints, const FChaosDebugDrawSettings* Settings = nullptr);
		CHAOS_API void DrawCCDAxisThreshold(const FVec3& X, const FVec3& AxisThreshold, const FVec3& DeltaX, const FQuat& R);
		CHAOS_API void DrawCCDCollisionImpulse(const FRigidTransform3& SpaceTransform, const FCCDConstraint& CCDConstraint, const int32 ManifoldPointIndex, const FVec3& Impulse, const FChaosDebugDrawSettings* Settings = nullptr);


		UE_DEPRECATED(5.4, "Pass in the ShapeInstance or call DrawParticleShapes")
		inline void DrawShape(const FRigidTransform3& ShapeTransform, const FImplicitObject* Implicit, const FShapeOrShapesArray& Shapes, const FColor& Color, const FChaosDebugDrawSettings* Settings = nullptr)
		{
			DrawShape(ShapeTransform, Implicit, Shapes.IsSingleShape() ? Shapes.GetShape() : nullptr, Color, 0.0f, Settings);
		}

		UE_DEPRECATED(5.4, "Pass in the ShapeInstance or call DrawParticleShapes")
		inline void DrawShape(const FRigidTransform3& ShapeTransform, const FImplicitObject* Implicit, const FShapeOrShapesArray& Shapes, const FColor& Color, const float Duration, const FChaosDebugDrawSettings* Settings = nullptr)
		{
			DrawShape(ShapeTransform, Implicit, Shapes.IsSingleShape() ? Shapes.GetShape() : nullptr, Color, Duration, Settings);
		}

		UE_DEPRECATED(5.5, "Now requires SpaceTransform as first parameter")
		inline void DrawSimulationSpace(const FSimulationSpace& SimSpace, const FChaosDebugDrawSettings* Settings = nullptr)
		{
		}
#endif
	}
}
