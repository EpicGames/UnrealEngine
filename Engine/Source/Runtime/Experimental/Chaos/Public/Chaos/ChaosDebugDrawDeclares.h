// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/Declares.h"
#include "Chaos/GeometryParticlesfwd.h"
#include "Chaos/Island/IslandManagerFwd.h"
#include "Misc/Build.h"

namespace Chaos
{
	class FConstraintHandle;

	class FPBDCollisionConstraints;

	class FPBDCollisionConstraintHandle;

	class FPBDConstraintColor;

	class FPBDJointConstraintHandle;

	class FPBDJointConstraints;

	class FSimulationSpace;

	class FCharacterGroundConstraintContainer;
}
