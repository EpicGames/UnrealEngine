// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "Math/MathFwd.h"

namespace Chaos
{
	class UE_INTERNAL IDebugDrawInterface
	{
	public:
		// Define an interface for rendering a mesh
		// DrawMesh use this interface to access the mesh details and will convert it to lines / solid renderable primitive based on state settings ( see above)
		struct IDebugDrawMesh
		{
			virtual ~IDebugDrawMesh() = default;

			virtual int32 GetMaxVertexIndex() const = 0;
			virtual bool IsValidVertex(int32 VertexIndex) const = 0;
			virtual FVector GetVertexPosition(int32 VertexIndex) const = 0;
			virtual FVector GetVertexNormal(int32 VertexIndex) const = 0;

			virtual int32 GetMaxTriangleIndex() const = 0;
			virtual bool IsValidTriangle(int32 TriangleIndex) const = 0;
			virtual FIntVector3 GetTriangle(int32 TriangleIndex) const = 0;
		};

		virtual ~IDebugDrawInterface() = default;

		// State management
		// values are set once and apply to any Draw*() calls
		virtual void SetColor(const FLinearColor& InColor) = 0;
		virtual void SetPointSize(float Size) = 0;
		virtual void SetLineWidth(double Width) = 0;
		virtual void SetWireframe(bool bInWireframe) = 0;
		virtual void SetShaded(bool bInShaded) = 0;
		virtual void SetTranslucent(bool bInShadedTranslucent) = 0;
		virtual void SetForegroundPriority() = 0;
		virtual void SetWorldPriority() = 0;
		virtual void ResetAllState() = 0;

		virtual void ReservePoints(int32 NumAdditionalPoints) = 0;

		// Draw methods
		virtual void DrawPoint(const FVector& Position) = 0;
		virtual void DrawLine(const FVector& Start, const FVector& End) const = 0;
		virtual void DrawText3d(const FString& String, const FVector& Location) const = 0;
		virtual void DrawBox(const FVector& Extents, const FQuat& Rotation, const FVector& Center, double UniformScale) const = 0;
		virtual void DrawSphere(const FVector& Center, double Radius) const = 0;
		virtual void DrawCapsule(const FVector& Center, const double& Radius, const double& HalfHeight, const FVector& XAxis, const FVector& YAxis, const FVector& ZAxis) const = 0;
		virtual void DrawMesh(const IDebugDrawMesh& Mesh) const = 0;
	};
}
