// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ChaosDebugNameDefines.h"
#include "Containers/UnrealString.h"
#include "Templates/SharedPointer.h"

#define UE_API CHAOS_API

namespace Chaos
{
	/**
	* A wrapper around shared pointer to a string that compiles away in shipping buiilds,
	* but Get() always returns a valid string reference to make writing logs a little easier
	*/
	class FSharedDebugName
	{
	public:
		static FSharedDebugName Make(const TCHAR* InName)
		{
			return FSharedDebugName(InName);
		}

		template<typename... ArgTypes>
		static FSharedDebugName Make(const TCHAR* InFmt, ArgTypes... Args)
		{
			FStringFormatOrderedArguments ArgsArray = { std::forward<ArgTypes>(Args)... };
			return FSharedDebugName(FString::Format(InFmt, ArgsArray));
		}


		FSharedDebugName() = default;
		FSharedDebugName(const FSharedDebugName& Other) = default;
		FSharedDebugName(FSharedDebugName&& Other) = default;
		FSharedDebugName& operator=(const FSharedDebugName& Other) = default;
		FSharedDebugName& operator=(FSharedDebugName&& Other) = default;

		FSharedDebugName(const FString& S);
		FSharedDebugName(FString&& S);

		bool IsValid() const;
		const FString& Value() const;
		const TSharedPtr<FString, ESPMode::ThreadSafe>& SharedValue() const;

	private:
#if CHAOS_DEBUG_NAME
		TSharedPtr<FString, ESPMode::ThreadSafe> Name;
#endif
		static UE_API TSharedPtr<FString, ESPMode::ThreadSafe> DefaultName;
	};

#if CHAOS_DEBUG_NAME
	inline FSharedDebugName::FSharedDebugName(const FString& S)
		: Name(TSharedPtr<FString, ESPMode::ThreadSafe>(new FString(S)))
	{
	}

	inline FSharedDebugName::FSharedDebugName(FString&& S)
		: Name(TSharedPtr<FString, ESPMode::ThreadSafe>(new FString(MoveTemp(S))))
	{
	}

	inline bool FSharedDebugName::IsValid() const
	{
		return Name.IsValid();
	}

	inline const FString& FSharedDebugName::Value() const
	{
		if (Name.IsValid())
		{
			return *Name.Get();
		}
		return *DefaultName.Get();
	}

	inline const TSharedPtr<FString, ESPMode::ThreadSafe>& FSharedDebugName::SharedValue() const
	{
		return Name;
	}
#else
	inline FSharedDebugName::FSharedDebugName(const FString& S) {}
	inline FSharedDebugName::FSharedDebugName(FString&& S) {}
	inline bool FSharedDebugName::IsValid() const { return false; }
	inline const FString& FSharedDebugName::Value() const { return *DefaultName.Get(); }
	inline const TSharedPtr<FString, ESPMode::ThreadSafe>& FSharedDebugName::SharedValue() const { return DefaultName; }
#endif
}

#undef UE_API
