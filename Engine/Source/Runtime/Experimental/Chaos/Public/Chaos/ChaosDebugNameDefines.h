// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifndef CHAOS_DEBUG_NAME
#define CHAOS_DEBUG_NAME 0
#endif

#ifndef CHAOS_SOLVER_DEBUG_NAME
#define CHAOS_SOLVER_DEBUG_NAME CHAOS_DEBUG_NAME || WITH_CHAOS_VISUAL_DEBUGGER
#endif