// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/ChaosCoreLogUtil.h"
#include "Chaos/Particle/ObjectState.h"
#include "Containers/UnrealString.h"

class FString;

namespace Chaos
{
	CHAOS_API FString ToString(const EObjectStateType ObjectState);
}