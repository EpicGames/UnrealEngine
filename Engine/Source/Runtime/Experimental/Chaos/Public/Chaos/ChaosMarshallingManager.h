// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/SpscQueue.h"
#include "Chaos/Defines.h"
#include "Chaos/ParticleDirtyFlags.h"
#include "Chaos/Framework/PhysicsProxyBase.h"
#include "Chaos/ParallelFor.h"
#include "PhysicsProxy/SingleParticlePhysicsProxyFwd.h"
#include "Chaos/SimCallbackObject.h"
#include "Chaos/AsyncInitBodyHelper.h"
#include "Framework/Threading.h"

class FGeometryCollectionResults;

namespace Chaos
{
class FDirtyPropertiesManager;
class FPullPhysicsData;

struct FDirtyProxy
{
	IPhysicsProxyBase* Proxy;
	FDirtyChaosProperties PropertyData;
	TArray<int32> ShapeDataIndices;

	FDirtyProxy(IPhysicsProxyBase* InProxy)
		: Proxy(InProxy)
	{
	}

	void SetDirtyIdx(int32 Idx)
	{
		Proxy->SetDirtyIdx(Idx);
	}

	void AddShape(int32 ShapeDataIdx)
	{
		ShapeDataIndices.Add(ShapeDataIdx);
	}

	void Clear(FDirtyPropertiesManager& Manager,int32 DataIdx,FShapeDirtyData* ShapesData)
	{
		PropertyData.Clear(Manager,DataIdx);
		for(int32 ShapeDataIdx : ShapeDataIndices)
		{
			ShapesData[ShapeDataIdx].Clear(Manager,ShapeDataIdx);
		}
	}

	void MoveData(FDirtyPropertiesManager& Manager, int32 FromIdx, int32 ToIdx)
	{
		PropertyData.MoveData(Manager, FromIdx, ToIdx);
	}
};

struct FDirtyProxiesBucket
{
	TArray<FDirtyProxy> ProxiesData;
};

class FDirtySet
{
public:
	void Add(IPhysicsProxyBase* Base)
	{
		if(Base->GetDirtyIdx() == INDEX_NONE)
		{
			FDirtyProxiesBucket& Bucket = DirtyProxyBuckets[(uint32)Base->GetType()];
			++DirtyProxyBucketInfo.Num[(uint32)Base->GetType()];
			++DirtyProxyBucketInfo.TotalNum;

			const int32 Idx = Bucket.ProxiesData.Num();
			Base->SetDirtyIdx(Idx);
			Bucket.ProxiesData.Add(Base);
		}
	}

	// Batch proxy insertion, does not check DirtyIdx.
	// Assumes proxies are the same type
	template< typename TProxiesArray>
	void AddMultipleUnsafe(TProxiesArray& ProxiesArray)
	{
		if(ProxiesArray.Num())
		{
			FDirtyProxiesBucket& Bucket = DirtyProxyBuckets[(uint32)ProxiesArray[0]->GetType()];
			int32 Idx = Bucket.ProxiesData.Num();
			Bucket.ProxiesData.Append(ProxiesArray);

			for (IPhysicsProxyBase* Proxy : ProxiesArray)
			{
				Proxy->SetDirtyIdx(Idx++);
				ensure(ProxiesArray[0]->GetType() == Proxy->GetType());
			}

			DirtyProxyBucketInfo.Num[(uint32)ProxiesArray[0]->GetType()] += ProxiesArray.Num();
			DirtyProxyBucketInfo.TotalNum += ProxiesArray.Num();
		}
		
	}

	// Forcefully removes the proxy from being dirty.
	void Remove(IPhysicsProxyBase* Base)
	{
		const int32 Idx = Base->GetDirtyIdx();
		if(Idx != INDEX_NONE)
		{
			FDirtyProxiesBucket& Bucket = DirtyProxyBuckets[(uint32)Base->GetType()];
			if(Idx == Bucket.ProxiesData.Num() - 1)
			{
				//last element so just pop
				Bucket.ProxiesData.Pop(EAllowShrinking::No);
				--DirtyProxyBucketInfo.Num[(uint32)Base->GetType()];
				--DirtyProxyBucketInfo.TotalNum;
			}
			else if(Bucket.ProxiesData.IsValidIndex(Idx))
			{
				//update other proxy's idx
				Bucket.ProxiesData.RemoveAtSwap(Idx);
				Bucket.ProxiesData[Idx].SetDirtyIdx(Idx);
				--DirtyProxyBucketInfo.Num[(uint32)Base->GetType()];
				--DirtyProxyBucketInfo.TotalNum;
			}

			Base->ResetDirtyIdx();
		}
	}

	/** Forcefully removes the proxy from being dirty without relying on DirtyIdx
	* NOTE: Does not reset IPhysicsProxyBase::DirtyIdx and also doesn't update the DirtyIdx for a moved entries IPhysicsProxyBase, both of which FDirtySet::Remove does */
	void RemoveWithoutDirtyIdx(IPhysicsProxyBase* Base, FDirtyPropertiesManager& Manager)
	{
		FDirtyProxiesBucket& Bucket = DirtyProxyBuckets[(uint32)Base->GetType()];
		for (int32 Idx = Bucket.ProxiesData.Num() - 1; Idx >= 0; Idx--)
		{
			if (Bucket.ProxiesData[Idx].Proxy == Base)
			{
				const int32 LastIdx = Bucket.ProxiesData.Num() - 1;

				// Clear data from manager
				Bucket.ProxiesData[Idx].Clear(Manager, Idx, GetShapesDirtyData());

				if (Idx == LastIdx)
				{
					// Last element, pop to remove it
					Bucket.ProxiesData.Pop(EAllowShrinking::No);
				}
				else
				{
					// Remove and swap with last element, update moved proxy's idx
					Bucket.ProxiesData.RemoveAtSwap(Idx);

					// Move data in manager for the swapped element
					Bucket.ProxiesData[Idx].MoveData(Manager, /*FromIdx*/LastIdx, /*ToIdx*/Idx);
				}

				--DirtyProxyBucketInfo.Num[(uint32)Base->GetType()];
				--DirtyProxyBucketInfo.TotalNum;

				// There is only one entry per proxy
				break;
			}
		}
	}

	// Only does the removal if no shapes are dirty.
	void RemoveIfNoShapesAreDirty(IPhysicsProxyBase* Base)
	{
		const int32 Idx = Base->GetDirtyIdx();
		if (Idx != INDEX_NONE)
		{
			FDirtyProxiesBucket& Bucket = DirtyProxyBuckets[(uint32)Base->GetType()];
			if (Bucket.ProxiesData.IsValidIndex(Idx))
			{
				FDirtyProxy& ProxyData = Bucket.ProxiesData[Idx];
				if (ProxyData.ShapeDataIndices.IsEmpty())
				{
					Remove(Base);
				}
			}
		}
	}

	void Reset()
	{
		for(FDirtyProxiesBucket& Bucket : DirtyProxyBuckets)
		{
			Bucket.ProxiesData.Reset();
		}
		
		DirtyProxyBucketInfo.Reset();
		ShapesData.Reset();
	}

	const FDirtyProxiesBucketInfo& GetDirtyProxyBucketInfo() const { return DirtyProxyBucketInfo; }
	int32 NumDirtyShapes() const { return ShapesData.Num(); }

	FShapeDirtyData* GetShapesDirtyData(){ return ShapesData.GetData(); }
	FDirtyProxy& GetDirtyProxyAt(EPhysicsProxyType ProxyType, int32 Idx) { return DirtyProxyBuckets[(uint32)ProxyType].ProxiesData[Idx]; }

	template <typename Lambda>
	void ParallelForEachProxy(const Lambda& Func)
	{
		::ParallelFor( TEXT("Chaos.PF"),DirtyProxyBucketInfo.TotalNum,1, [this,&Func](int32 Idx)
		{
			int32 BucketIdx, InnerIdx;
			DirtyProxyBucketInfo.GetBucketIdx(Idx, BucketIdx, InnerIdx);
			Func(InnerIdx, DirtyProxyBuckets[BucketIdx].ProxiesData[InnerIdx]);
		});
	}

	template <typename Lambda>
	void ParallelForEachProxy(const Lambda& Func) const
	{
		::ParallelFor(DirtyProxyBucketInfo.TotalNum,[this,&Func](int32 Idx)
		{
			int32 BucketIdx, InnerIdx;
			DirtyProxyBucketInfo.GetBucketIdx(Idx, BucketIdx, InnerIdx);
			Func(InnerIdx, DirtyProxyBuckets[BucketIdx].ProxiesData[InnerIdx]);
		});
	}

	template <typename Lambda>
	void ForEachProxy(const Lambda& Func)
	{
		for(int32 BucketIdx = 0; BucketIdx < (uint32)EPhysicsProxyType::Count; ++BucketIdx)
		{
			int32 Idx = 0;
			for (FDirtyProxy& Dirty : DirtyProxyBuckets[BucketIdx].ProxiesData)
			{
				Func(Idx++, Dirty);
			}
		}
	}

	template <typename Lambda>
	void ForEachProxy(const Lambda& Func) const
	{
		for (int32 BucketIdx = 0; BucketIdx < (uint32)EPhysicsProxyType::Count; ++BucketIdx)
		{
			int32 Idx = 0;
			for (const FDirtyProxy& Dirty : DirtyProxyBuckets[BucketIdx].ProxiesData)
			{
				Func(Idx++, Dirty);
			}
		}
	}

	void AddShape(IPhysicsProxyBase* Proxy,int32 ShapeIdx)
	{
		Add(Proxy);
		FDirtyProxy& Dirty = DirtyProxyBuckets[(uint32)Proxy->GetType()].ProxiesData[(uint32)Proxy->GetDirtyIdx()];
		for(int32 NewShapeIdx = Dirty.ShapeDataIndices.Num(); NewShapeIdx <= ShapeIdx; ++NewShapeIdx)
		{
			const int32 ShapeDataIdx = ShapesData.Add(FShapeDirtyData(NewShapeIdx));
			Dirty.AddShape(ShapeDataIdx);
		}
	}

	void SetNumDirtyShapes(IPhysicsProxyBase* Proxy,int32 NumShapes)
	{
		Add(Proxy);
		FDirtyProxy& Dirty = DirtyProxyBuckets[(uint32)Proxy->GetType()].ProxiesData[Proxy->GetDirtyIdx()];

		if(NumShapes < Dirty.ShapeDataIndices.Num())
		{
			Dirty.ShapeDataIndices.SetNum(NumShapes);
		} else
		{
			for(int32 NewShapeIdx = Dirty.ShapeDataIndices.Num(); NewShapeIdx < NumShapes; ++NewShapeIdx)
			{
				const int32 ShapeDataIdx = ShapesData.Add(FShapeDirtyData(NewShapeIdx));
				Dirty.AddShape(ShapeDataIdx);
			}
		}
	}

private:
	FDirtyProxiesBucketInfo DirtyProxyBucketInfo;
	FDirtyProxiesBucket DirtyProxyBuckets[(uint32)EPhysicsProxyType::Count];
	TArray<FShapeDirtyData> ShapesData;
};

class FChaosMarshallingManager;

struct FPushPhysicsData
{
	FDirtyPropertiesManager DirtyPropertiesManager;
	FDirtySet DirtyProxiesDataBuffer;
	FReal StartTime;
	FReal ExternalDt;
	int32 ExternalTimestamp;
	int32 InternalStep;		//The solver step this data will be associated with
	int32 IntervalStep;		//The step we are currently at for this simulation interval. If not sub-stepping both step and num steps are 1: step is [0, IntervalNumSteps-1]
	int32 IntervalNumSteps;	//The total number of steps associated with this simulation interval
	bool bSolverSubstepped;

	TArray<ISimCallbackObject*> SimCallbackObjectsToAdd;	//callback object registered at this specific time
	TArray<ISimCallbackObject*> SimCallbackObjectsToRemove;	//callback object removed at this specific time
	TArray<FSimCallbackInputAndObject> SimCallbackInputs; //set of callback inputs pushed at this specific time
	TArray<FSimCallbackCommandObject*> SimCommands;	//commands to run (this is a one off command)

	void Reset();
	void ResetForHistory();
	void ResetDirtyProxiesBuffer();
	void ClearAsyncInputs();
	void ClearSimCommands();
	void ClearAsyncInputForCallback(ISimCallbackObject* Callback);
	void CopySubstepData(const FPushPhysicsData& FirstStepData);
};

/** Manages data that gets marshaled from GT to PT using a timestamp
*/
class FChaosMarshallingManager
{
public:
	CHAOS_API FChaosMarshallingManager();
	CHAOS_API ~FChaosMarshallingManager();

	/** Grabs the producer data to write into. Should only be called by external thread */
	FPushPhysicsData* GetProducerData_External()
	{
		return ProducerData;
	}

	/** Grabs the consumer data to read from. Should only be called by internal thread */
	FPushPhysicsData* GetConsumerData_Internal()
	{
		return ConsumerData;
	}

	/** Sets a new internal consumer data to read from. Should only be called by internal thread */
	void SetConsumerData_Internal(FPushPhysicsData* InPushData)
	{
		ConsumerData = InPushData;
	}

	void AddDirtyProxy(IPhysicsProxyBase* ProxyBaseIn)
	{
		UE_CHAOS_ASYNC_INITBODY_WRITESCOPELOCK(MarshallingManagerLock);
		ProducerData->DirtyProxiesDataBuffer.Add(ProxyBaseIn);
	}

	void RemoveDirtyProxy(IPhysicsProxyBase* ProxyBaseIn)
	{
		UE_CHAOS_ASYNC_INITBODY_WRITESCOPELOCK(MarshallingManagerLock);
		ProducerData->DirtyProxiesDataBuffer.Remove(ProxyBaseIn);
	}

	void RemoveDirtyProxyFromHistory_Internal(IPhysicsProxyBase* ProxyBaseIn)
	{
		// Clear any dirty data from push data in history since we can't recreate a deleted physics proxy or particle during resimulation
		if (HistoryQueue_Internal.Num() > 0)
		{
			if (ConsumerData)
			{
				// Remove from current PushData not yet put into history
				ConsumerData->DirtyProxiesDataBuffer.RemoveWithoutDirtyIdx(ProxyBaseIn, ConsumerData->DirtyPropertiesManager);
			}

			// Remove from history PusData
			for (FPushPhysicsData* PushDataHistoryEntry : HistoryQueue_Internal)
			{
				PushDataHistoryEntry->DirtyProxiesDataBuffer.RemoveWithoutDirtyIdx(ProxyBaseIn, PushDataHistoryEntry->DirtyPropertiesManager);
			}
		}
	}

	void RemoveDirtyProxyIfNoShapesAreDirty(IPhysicsProxyBase* ProxyBaseIn)
	{
		UE_CHAOS_ASYNC_INITBODY_WRITESCOPELOCK(MarshallingManagerLock);
		ProducerData->DirtyProxiesDataBuffer.RemoveIfNoShapesAreDirty(ProxyBaseIn);
	}

	const FDirtyProxiesBucketInfo& GetDirtyProxyBucketInfo_External()
	{
		ensureAlwaysMsgf(!Chaos::CVars::bEnableAsyncInitBody, TEXT("This method is not safe when p.Chaos.EnableAsyncInitBody is true"));
		return ProducerData->DirtyProxiesDataBuffer.GetDirtyProxyBucketInfo();
	}

	int32 GetDirtyProxyBucketInfoNum_External(EPhysicsProxyType Type)
	{
		UE_CHAOS_ASYNC_INITBODY_READSCOPELOCK(MarshallingManagerLock);
		return ProducerData->DirtyProxiesDataBuffer.GetDirtyProxyBucketInfo().Num[(uint32)Type];
	}

	// Batch dirty proxies without checking DirtyIdx.
	template <typename TProxiesArray>
	void AddDirtyProxiesUnsafe(TProxiesArray& ProxiesArray)
	{
		UE_CHAOS_ASYNC_INITBODY_WRITESCOPELOCK(MarshallingManagerLock);
		ProducerData->DirtyProxiesDataBuffer.AddMultipleUnsafe(ProxiesArray);
	}

	void AddDirtyProxyShape(IPhysicsProxyBase* ProxyBaseIn, int32 ShapeIdx)
	{
		UE_CHAOS_ASYNC_INITBODY_WRITESCOPELOCK(MarshallingManagerLock);
		ProducerData->DirtyProxiesDataBuffer.AddShape(ProxyBaseIn, ShapeIdx);
	}

	void SetNumDirtyShapes(IPhysicsProxyBase* Proxy, int32 NumShapes)
	{
		UE_CHAOS_ASYNC_INITBODY_WRITESCOPELOCK(MarshallingManagerLock);
		ProducerData->DirtyProxiesDataBuffer.SetNumDirtyShapes(Proxy, NumShapes);
	}

	void RegisterSimCallbackObject_External(ISimCallbackObject* SimCallbackObject)
	{
		UE_CHAOS_ASYNC_INITBODY_WRITESCOPELOCK(MarshallingManagerLock);
		ProducerData->SimCallbackObjectsToAdd.Add(SimCallbackObject);
	}

	void RegisterSimCommand_External(FSimCallbackCommandObject* SimCommand)
	{
		UE_CHAOS_ASYNC_INITBODY_WRITESCOPELOCK(MarshallingManagerLock);
		ProducerData->SimCommands.Add(SimCommand);
	}

	void EnqueueCommandScheduled_Internal(FSimCallbackCommandObject* SimCommand)
	{
		SimCommandsScheduled_Internal.Add(SimCommand);
	}

	TArray<FSimCallbackCommandObject*>& GetSimCommandsScheduled_Internal()
	{
		return SimCommandsScheduled_Internal;
	}

	void UnregisterSimCallbackObject_External(ISimCallbackObject* SimCallbackObject)
	{
		UE_CHAOS_ASYNC_INITBODY_WRITESCOPELOCK(MarshallingManagerLock);
		SimCallbackObject->bPendingDelete_External = true;
		ProducerData->SimCallbackObjectsToRemove.Add(SimCallbackObject);
	}

	void AddSimCallbackInputData_External(ISimCallbackObject* SimCallbackObject, FSimCallbackInput* InputData)
	{
		ProducerData->SimCallbackInputs.Add(FSimCallbackInputAndObject{ SimCallbackObject, InputData });
	}
	/** Step forward using the external delta time. Should only be called by external thread */
	CHAOS_API void Step_External(FReal ExternalDT, const int32 NumSteps = 1, bool bSolverSubstepped = false);

	/** Step the internal time forward if possible*/
	CHAOS_API FPushPhysicsData* StepInternalTime_External();

	/** Frees the push data back into the pool. Internal thread should call this when finished processing data*/
	CHAOS_API void FreeData_Internal(FPushPhysicsData* PushData);

	/** May record data to history, or may free immediately depending on rewind needs. Either way you should assume data is gone after calling this */
	CHAOS_API void FreeDataToHistory_Internal(FPushPhysicsData* PushData);

	/** Frees the pull data back into the pool. External thread should call this when finished processing data*/
	CHAOS_API void FreePullData_External(FPullPhysicsData* PullData);

	/** Returns the timestamp associated with inputs enqueued. */
	int32 GetExternalTimestamp_External() const { return ExternalTimestamp_External; }

	/** Returns the amount of external time pushed so far. Any external commands or events should be associated with this time */
	FReal GetExternalTime_External() const { return ExternalTime_External; }

	/** Returns the internal step that the current PushData will be associated with once it is marshalled over*/
	int32 GetInternalStep_External() const { return InternalStep_External; }

	/** Used to delay marshalled data. This is mainly used for testing at the moment */
	void SetTickDelay_External(int32 InDelay) { Delay = InDelay; }

	/** Returns the current pull data being written to. This holds the results of dirty data to be read later by external thread*/
	FPullPhysicsData* GetCurrentPullData_Internal() { return CurPullData; }

	/** Hands pull data off to external thread */
	CHAOS_API void FinalizePullData_Internal(int32 LatestExternalTimestampConsumed, FReal SimStartTime, FReal DeltaTime);

	/** Pops and returns the earliest pull data available. nullptr means results are not ready or no work is pending */
	FPullPhysicsData* PopPullData_External()
	{
		FPullPhysicsData* Result = nullptr;
		PullDataQueue.Dequeue(Result);
		return Result;
	}

	CHAOS_API void SetHistoryLength_Internal(int32 InHistoryLength);

	/** Returns the history buffer of the latest NumFrames. The history that comes before these frames is discarded*/
	CHAOS_API TArray<FPushPhysicsData*> StealHistory_Internal(int32 NumFrames);

	/** Debug and print the history buffer of the latest NumFrames. */
	CHAOS_API void DebugHistory_Internal(int32 NumFrames);

	/** Return the size of the history queue */
	int32 GetNumHistory_Internal() const {return HistoryQueue_Internal.Num();}

	/** Used for multithreaded access */
	FRWLock& GetMarshallingManagerLock() { return MarshallingManagerLock; }

private:
	std::atomic<FReal> ExternalTime_External;	//the global time external thread is currently at
	std::atomic<int32> ExternalTimestamp_External; //the global timestamp external thread is currently at (1 per frame)
	std::atomic<FReal> SimTime_External;	//the global time the sim is at (once Step_External is called this time advances, even though the actual sim work has yet to be done)
	std::atomic<int32> InternalStep_External; //the current internal step we are pushing work into. This is not synced with external timestamp as we may sub-step. This should match the solver step
	
	//push
	FPushPhysicsData* ConsumerData; // The current internal PushData 
	FPushPhysicsData* ProducerData; // The external PushData before it's handed over to ExternalQueue
	TArray<FPushPhysicsData*> ExternalQueue;	//the data pushed from external thread with a time stamp
	TSpscQueue<FPushPhysicsData*> PushDataPool;	//pool to grab more push data from to avoid expensive reallocs
	TArray<TUniquePtr<FPushPhysicsData>> BackingBuffer;	//all push data is cleaned up by this
	TArray<FPushPhysicsData*> HistoryQueue_Internal;	//all push data still needed for potential rewinds. Latest data comes first
	TArray<FSimCallbackCommandObject*> SimCommandsScheduled_Internal; // SimCommands scheduled for a future frame, will be inserted into PushData on the correct frame

	//pull
	FPullPhysicsData* CurPullData;	//the current pull data sim is writing to
	TPhysicsSpscQueue<FPullPhysicsData*> PullDataQueue;	//the results the simulation has written to. Consumed by external thread
	TPhysicsSpscQueue<FPullPhysicsData*> PullDataPool;	//the pull data pool to avoid reallocs. Pushed by external thread, popped by internal
	TArray<TUniquePtr<FPullPhysicsData>> BackingPullBuffer;		//all pull data is cleaned up by this

	std::atomic<int32> Delay;

	int32 HistoryLength;	//how long to keep push data for

	CHAOS_API void PrepareExternalQueue_External();
	CHAOS_API void PreparePullData();

	/** Used for multithreaded access */
	FRWLock MarshallingManagerLock;
};
}; // namespace Chaos
