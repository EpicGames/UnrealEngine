// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Misc/OutputDevice.h"
#include "ProfilingDebugging/ScopedTimers.h"

#ifndef CHAOS_PERF_TEST_ENABLED
#define CHAOS_PERF_TEST_ENABLED 1
#endif

#if CHAOS_PERF_TEST_ENABLED
enum class EChaosPerfUnits
{
	S,
	Ms,
	Us,
	Num
};

class FChaosScopedDurationTimeLogger
{
public:
	explicit FChaosScopedDurationTimeLogger(const TCHAR* InLabel)
		: Label(InLabel)
		, Device(GLog)
		, Accumulator(0.0)
		, Timer(Accumulator)
	{
		Timer.Start();
	}

	~FChaosScopedDurationTimeLogger()
	{
		static const float Multipliers[static_cast<int>(EChaosPerfUnits::Num)] = { 1.f, 1000.f, 1000000.f };
		static const TCHAR* Units[static_cast<int>(EChaosPerfUnits::Num)] = { TEXT("s"), TEXT("ms"), TEXT("us") };
		Timer.Stop();
		if (GlobalLabel)
		{
			//Device->Logf(TEXT("%s - %s: %4.fus"), GlobalLabel, Label, Accumulator * 1000000.f);
			Device->Logf(TEXT("%s - %s: %f%s"), GlobalLabel, Label, Accumulator * Multipliers[static_cast<int>(GlobalUnits)], Units[static_cast<int>(GlobalUnits)]);
		}
	}

	static CHAOS_API const TCHAR* GlobalLabel;

	static CHAOS_API EChaosPerfUnits GlobalUnits;

	const TCHAR*   Label;
	FOutputDevice* Device;
	double         Accumulator;
	FDurationTimer Timer;
};

struct FScopedChaosPerfTest
{
	FScopedChaosPerfTest(const TCHAR* InLabel, EChaosPerfUnits Units)
	{
		FChaosScopedDurationTimeLogger::GlobalLabel = InLabel;
		FChaosScopedDurationTimeLogger::GlobalUnits = Units;
	}
	~FScopedChaosPerfTest() { FChaosScopedDurationTimeLogger::GlobalLabel = nullptr; }
};

#define CHAOS_PERF_TEST(x, units) FScopedChaosPerfTest Scope_##x(TEXT(#x), units);
#define CHAOS_SCOPED_TIMER(x) FChaosScopedDurationTimeLogger Timer_##x(TEXT(#x));
#else
#define CHAOS_PERF_TEST(x, units)
#define CHAOS_SCOPED_TIMER(x)
#endif
