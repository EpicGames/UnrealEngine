// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Character/CharacterGroundConstraintSettings.h"
#include "Chaos/PBDConstraintBaseData.h"
#include "Chaos/ParticleDirtyFlags.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"

namespace Chaos
{
	/// Game thread side representation of a character ground constraint
	/// Modified data from this class will be pushed automatically
	/// to the physics thread and available through the
	/// FCharacterGroundConstraintHandle class.
	/// The FCharacterGroundConstraintProxy class, accessible via
	/// GetProxy(), knows about both representations and marshalls
	/// the data between them
	class FCharacterGroundConstraint : public FConstraintBase
	{
	public:
		using Base = FConstraintBase;
		friend class FCharacterGroundConstraintProxy; // For setting the output data

		CHAOS_API FCharacterGroundConstraint();
		virtual ~FCharacterGroundConstraint() override = default;

		// The constraint requires a character body, which is fixed, but the ground body is
		// optional and can be changed dynamically using SetGroundParticleProxy
		CHAOS_API void Init(FSingleParticlePhysicsProxy* InCharacterProxy);

		//////////////////////////////////////////////////////////////////////////
		// Particle proxy

		FSingleParticlePhysicsProxy* GetCharacterParticleProxy() const
		{
			return CharacterProxy.Read().ParticleProxy;
		}

		FSingleParticlePhysicsProxy* GetGroundParticleProxy() const
		{
			return GroundProxy.Read().ParticleProxy;
		}

		void SetGroundParticleProxy(FSingleParticlePhysicsProxy* InGroundProxy)
		{
			GroundProxy.Modify(/*bInvalidate=*/true, DirtyFlags, Proxy, [InGroundProxy](FParticleProxyProperty& Data)
				{
					Data.ParticleProxy = InGroundProxy;
				});
			if (InGroundProxy)
			{
				WakeGroundBody();
			}
		}

		//////////////////////////////////////////////////////////////////////////
		// Settings accessors

		/// Unit normal to the ground plane
		/// Input vector is normalized if it is not already a unit vector
		void SetGroundNormal(const FVec3& Normal)
		{
			FVec3 Tmp = Normal;
			FReal SizeSq = Tmp.SizeSquared();
			if (!FMath::IsNearlyEqual(SizeSq, FReal(1.0)))
			{
				Tmp *= FMath::InvSqrt(SizeSq);
			}
			ConstraintData.Modify(true, DirtyFlags, Proxy, [&Tmp](FCharacterGroundConstraintDynamicData& Data) {
				Data.GroundNormal = Tmp;
				});
		}
		FVec3 GetGroundNormal() const { return ConstraintData.Read().GroundNormal; }

		/// Unit vector defining the character up direction
		/// Input vector is normalized if it is not already a unit vector
		void SetVerticalAxis(const FVec3& Axis)
		{
			FVec3 Tmp = Axis;
			FReal SizeSq = Tmp.SizeSquared();
			if (!FMath::IsNearlyEqual(SizeSq, FReal(1.0)))
			{
				Tmp *= FMath::InvSqrt(SizeSq);
			}
			ConstraintSettings.Modify(true, DirtyFlags, Proxy, [&Tmp](FCharacterGroundConstraintSettings& Data) {
				Data.VerticalAxis = Tmp;
				});
		}
		FVec3 GetVerticalAxis() const { return ConstraintSettings.Read().VerticalAxis; }

		/// Vertical distance from the character body to the ground
		/// This should be set every frame as the character moves over the ground and is typically
		/// measured by performing one or more physics traces
		/// Input should be positive
		void SetGroundDistance(const FReal& Value)
		{
			ConstraintData.Modify(true, DirtyFlags, Proxy, [&Value](FCharacterGroundConstraintDynamicData& Data) {
				Data.GroundDistance = Value > FReal(0.0) ? Value : FReal(0.0);
				});
			WakeCharacterBody();
		}
		FReal GetGroundDistance() const { return ConstraintData.Read().GroundDistance; }

		/// Target vertical distance from the character body to the ground
		/// If the ground distance is smaller than this number the constraint will
		/// try to fix up the error to keep the character at the target height
		/// Input should be positive
		void SetTargetHeight(const FReal& Value)
		{
			ConstraintSettings.Modify(true, DirtyFlags, Proxy, [&Value](FCharacterGroundConstraintSettings& Data) {
				Data.TargetHeight = Value > FReal(0.0) ? Value : FReal(0.0);
				});
		}
		FReal GetTargetHeight() const { return ConstraintSettings.Read().TargetHeight; }

		/// The damping factor is used to add softness to the vertical constraint between the character
		/// and the ground. This can add springiness to landings and slow the rate of step ups.
		/// Input should be positive. A value of 0 gives rigid behavior. A value of 1/Dt or above will
		/// give maximum softness
		/// Units are /T
		void SetDampingFactor(const FReal& Value)
		{
			ConstraintSettings.Modify(true, DirtyFlags, Proxy, [&Value](FCharacterGroundConstraintSettings& Data) {
				Data.DampingFactor = Value > FReal(0.0) ? Value : FReal(0.0);
				});
		}
		FReal GetDampingFactor() const { return ConstraintSettings.Read().DampingFactor; }

		/// This is a fractional value used to scale the ground mass in the motion target constraint.
		/// It controls how much tangential force is applied to the ground when the character is moving.
		/// 1 means the full reaction force is applied
		/// 0 means zero reaction force is applied
		void SetMotionTargetMassBias(const FReal& Value)
		{
			ConstraintSettings.Modify(true, DirtyFlags, Proxy, [&Value](FCharacterGroundConstraintSettings& Data) {
				Data.MotionTargetMassBias = FMath::Clamp(Value, 0.0, 1.0);
				});
		}
		FReal GetMotionTargetMassBias() const { return ConstraintSettings.Read().MotionTargetMassBias; }

		/// This is a fractional value used to scale the radial force so that the character can achieve
		/// the motion target
		/// 1 means full scaling is applied and the character will always be able to achieve the motion target
		/// 0 means no scaling is applied to the radial force limit
		void SetRadialForceMotionTargetScaling(const FReal& Value)
		{
			ConstraintSettings.Modify(true, DirtyFlags, Proxy, [&Value](FCharacterGroundConstraintSettings& Data) {
				Data.RadialForceMotionTargetScaling = FMath::Clamp(Value, 0.0, 1.0);
				});
		}
		FReal GetRadialForceMotionTargetScaling() const { return ConstraintSettings.Read().RadialForceMotionTargetScaling; }

		/// This is the height below which the character is assumed to be on the ground and can apply
		/// force to move. This is useful if the character is slightly off the ground but should still
		/// be considered in a locomotion rather than falling state.
		/// If this number is set to zero then the constraint can only apply force to move to the motion
		/// target if the solver determines that there is a ground reaction force
		/// Input should be positive
		void SetAssumedOnGroundHeight(const FReal& Value)
		{
			ConstraintSettings.Modify(true, DirtyFlags, Proxy, [&Value](FCharacterGroundConstraintSettings& Data) {
				Data.AssumedOnGroundHeight = Value > FReal(0.0) ? Value : FReal(0.0);
				});
		}
		FReal GetAssumedOnGroundHeight() const { return ConstraintSettings.Read().AssumedOnGroundHeight; }

		/// The maximum angle of slope that is considered walkable.
		/// The slope angle is defined as the angle between the ground normal and vertical axis.
		/// If the slope angle is greater than the maximum walkable slope angle the motion target
		/// vector is adjusted to stop movement up the slope
		void SetMaxWalkableSlopeAngle(const FReal& Value)
		{
			ConstraintSettings.Modify(true, DirtyFlags, Proxy, [&Value](FCharacterGroundConstraintSettings& Data) {
				FReal Angle = FMath::Clamp(Value, 0.0, UE_HALF_PI);
				Data.CosMaxWalkableSlopeAngle = FMath::Cos(Angle);
				});
		}
		FReal GetMaxWalkableSlopeAngle() const { return FMath::Acos(ConstraintSettings.Read().CosMaxWalkableSlopeAngle); }

		void SetCosMaxWalkableSlopeAngle(const FReal& Value)
		{
			ConstraintSettings.Modify(true, DirtyFlags, Proxy, [&Value](FCharacterGroundConstraintSettings& Data) {
				FReal CosAngle = FMath::Clamp(Value, -1.0, 1.0);
				Data.CosMaxWalkableSlopeAngle = CosAngle;
				});
		}
		FReal GetCosMaxWalkableSlopeAngle() const { return ConstraintSettings.Read().CosMaxWalkableSlopeAngle; }

		/// Set the target change in position for the character. This is used to move the character
		/// along the ground, and this target will be clamped to the ground plane.
		/// The solver will apply a force to move the character to the desired position as long as
		/// the character is considered to be on the ground and the force required is less than or
		/// equal to the radial force limit. For motion off of the ground, e.g. jumping, set a
		/// velocity or impulse on the character body directly.
		/// This should be set every frame for a locomoting character, and the value is typically
		/// controlled either from a character movement model or from root motion extracted from
		/// an animation sequence.
		void SetTargetDeltaPosition(const FVec3& Value)
		{
			ConstraintData.Modify(true, DirtyFlags, Proxy, [&Value](FCharacterGroundConstraintDynamicData& Data) {
				Data.TargetDeltaPosition = Value;
				});
			WakeCharacterBody();
		}
		FVec3 GetTargetDeltaPosition() const { return ConstraintData.Read().TargetDeltaPosition; }

		/// Set the target orientation change about the vertical axis. This is used to turn the
		/// character when it is on the ground. The solver will do this by applying a torque if the
		/// character is considered to be on the ground and the torque required is less than or equal
		/// to the torque limit. For orientation change off the ground, e.g. in air steering, set an
		/// angular velocity or impulse on the character body directly.
		/// This should be set every frame for a locomoting character, and the value is typically
		/// controlled either from a character movement model or from root motion extracted from
		/// an animation sequence.
		void SetTargetDeltaFacing(const FReal& Value)
		{
			ConstraintData.Modify(true, DirtyFlags, Proxy, [&Value](FCharacterGroundConstraintDynamicData& Data) {
				Data.TargetDeltaFacing = Value;
				});
			WakeCharacterBody();
		}
		FReal GetTargetDeltaFacing() const { return ConstraintData.Read().TargetDeltaFacing; }

		///  Set both the target delta position and facing angle. This is more efficient than setting
		/// them both individually
		void SetMotionTarget(const FVector& TargetDeltaPosition, const FReal& TargetDeltaFacing)
		{
			ConstraintData.Modify(true, DirtyFlags, Proxy, [&TargetDeltaPosition, &TargetDeltaFacing](FCharacterGroundConstraintDynamicData& Data) {
				Data.TargetDeltaPosition = TargetDeltaPosition;
				Data.TargetDeltaFacing = TargetDeltaFacing;
				});
			WakeCharacterBody();
		}

		/// The maximum force that the character can apply in the plane defined by the ground normal
		/// to move the character to its target position (or hold the character in place if no target
		/// delta position is set).
		void SetRadialForceLimit(const FReal& Value)
		{
			ConstraintSettings.Modify(true, DirtyFlags, Proxy, [&Value](FCharacterGroundConstraintSettings& Data) {
				Data.RadialForceLimit = Value > FReal(0.0) ? Value : FReal(0.0);
				});
		}
		FReal GetRadialForceLimit() const { return ConstraintSettings.Read().RadialForceLimit; }

		/// The maximum force that the character can apply in the plane defined by the ground normal
		/// to hold the character in place while standing on an unwalkable incline (defined by max
		/// walkable slope angle)
		void SetFrictionForceLimit(const FReal& Value)
		{
			ConstraintSettings.Modify(true, DirtyFlags, Proxy, [&Value](FCharacterGroundConstraintSettings& Data) {
				Data.FrictionForceLimit = Value > FReal(0.0) ? Value : FReal(0.0);
				});
		}
		FReal GetFrictionForceLimit() const { return ConstraintSettings.Read().FrictionForceLimit; }

		/// The maximum torque that the character can apply about the vertical axis to rotate the
		/// character to its target facing direction (or hold the character in place if no target
		/// delta facing is set).
		void SetTwistTorqueLimit(const FReal& Value)
		{
			ConstraintSettings.Modify(true, DirtyFlags, Proxy, [&Value](FCharacterGroundConstraintSettings& Data) {
				Data.TwistTorqueLimit = Value > FReal(0.0) ? Value : FReal(0.0);
				});
		}
		FReal GetTwistTorqueLimit() const { return ConstraintSettings.Read().TwistTorqueLimit; }

		/// The maximum torque that the character can apply about the non vertical axes to keep
		/// the character upright
		void SetSwingTorqueLimit(const FReal& Value)
		{
			ConstraintSettings.Modify(true, DirtyFlags, Proxy, [&Value](FCharacterGroundConstraintSettings& Data) {
				Data.SwingTorqueLimit = Value > FReal(0.0) ? Value : FReal(0.0);
				});
		}
		FReal GetSwingTorqueLimit() const { return ConstraintSettings.Read().SwingTorqueLimit; }

		void SetUserData(void* Value)
		{
			ConstraintSettings.Modify(true, DirtyFlags, Proxy, [&Value](FCharacterGroundConstraintSettings& Data) {
				Data.UserData = Value;
				});
		}
		void* GetUserData() const { return ConstraintSettings.Read().UserData; }

		/// Maximum ratio between the character mass and the ground body mass as used by the constraint solver
		/// E.g. a value of 2 means that the character can be at most twice as heavy as the ground body and
		/// if it is over that the constraint solve will inflate the mass of the ground body to reach a 2:1 ratio
		/// This is useful for stability when walking on a light mass.
		/// A negative value disables this mass conditioning feature.
		void SetMaxCharacterGroundMassRatio(const FReal& Value)
		{
			ConstraintSettings.Modify(true, DirtyFlags, Proxy, [&Value](FCharacterGroundConstraintSettings& Data) {
				Data.MaxCharacterGroundMassRatio = Value;
				});
		}
		FReal GetMaxCharacterGroundMassRatio() const { return ConstraintSettings.Read().MaxCharacterGroundMassRatio; }

		//////////////////////////////////////////////////////////////////////////
		// Output

		// Get the force applied by the solver due to this constraint. Units are ML/T^2
		FVector GetSolverAppliedForce() const { return SolverAppliedForce; }

		// Get the torque applied by the solver due to this constraint. Units are ML^2/T^2
		FVector GetSolverAppliedTorque() const { return SolverAppliedTorque; }

	private:
		void WakeCharacterBody()
		{
			if (FSingleParticlePhysicsProxy* Character = CharacterProxy.Read().ParticleProxy)
			{
				FRigidBodyHandle_External& Body = Character->GetGameThreadAPI();
				if (Body.ObjectState() == EObjectStateType::Sleeping)
				{
					Body.SetObjectState(EObjectStateType::Dynamic);
				}
			}
		}

		void WakeGroundBody()
		{
			if (FSingleParticlePhysicsProxy* Ground = GroundProxy.Read().ParticleProxy)
			{
				FRigidBodyHandle_External& Body = Ground->GetGameThreadAPI();
				if (Body.ObjectState() == EObjectStateType::Sleeping)
				{
					Body.SetObjectState(EObjectStateType::Dynamic);
				}
			}
		}

		//////////////////////////////////////////////////////////////////////////
		// FConstraintBase implementation
		CHAOS_API virtual void SyncRemoteDataImp(FDirtyPropertiesManager& Manager, int32 DataIdx, FDirtyChaosProperties& RemoteData) override;

		TChaosProperty<FParticleProxyProperty, EChaosProperty::CharacterParticleProxy> CharacterProxy;
		TChaosProperty<FParticleProxyProperty, EChaosProperty::GroundParticleProxy> GroundProxy;
		TChaosProperty<FCharacterGroundConstraintSettings, EChaosProperty::CharacterGroundConstraintSettings> ConstraintSettings;
		TChaosProperty<FCharacterGroundConstraintDynamicData, EChaosProperty::CharacterGroundConstraintDynamicData> ConstraintData;

		friend class FCharacterGroundConstraintProxy; // for setting the force and torque from the physics thread data
		FVector SolverAppliedForce;
		FVector SolverAppliedTorque;
	};

} // namespace Chaos
