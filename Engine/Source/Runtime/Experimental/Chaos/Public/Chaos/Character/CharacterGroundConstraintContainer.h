// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"

#include "Chaos/Character/CharacterGroundConstraintSettings.h"
#include "Chaos/ConstraintHandle.h"
#include "Chaos/Evolution/IndexedConstraintContainer.h"
#include "Chaos/GeometryParticles.h"
#include "Chaos/PBDConstraintContainer.h"

namespace ChaosTest
{
	class CharacterGroundConstraintContainerTest;
}

namespace Chaos
{
	namespace CVars
	{
		CHAOS_API extern float Chaos_CharacterGroundConstraint_InputMovementThreshold;
		CHAOS_API extern float Chaos_CharacterGroundConstraint_ExternalMovementThreshold;
	}

	class FCharacterGroundConstraintContainer;
	namespace Private
	{
		class FCharacterGroundConstraintContainerSolver;
	}

	/// Physics thread side representation of a character ground constraint
	/// Data accessed through this class is synced from the game thread
	/// representation FCharacterGroundConstraint
	class FCharacterGroundConstraintHandle final : public TIntrusiveConstraintHandle<FCharacterGroundConstraintHandle>
	{
	public:
		using Base = TIntrusiveConstraintHandle<FCharacterGroundConstraintHandle>;
		using FConstraintContainer = FCharacterGroundConstraintContainer;

		FCharacterGroundConstraintHandle()
			: TIntrusiveConstraintHandle<FCharacterGroundConstraintHandle>()
		{
		}

		static const FConstraintHandleTypeID& StaticType()
		{
			static FConstraintHandleTypeID STypeID(TEXT("FCharacterGroundConstraintHandle"), &FIndexedConstraintHandle::StaticType());
			return STypeID;
		}

		ESyncState GetSyncState() const { return SyncState; }
		void SetSyncState(ESyncState InSyncState) { SyncState = InSyncState; }

		EResimType GetResimType() const { return ResimType; }
		bool GetEnabledDuringResim() const { return bEnabledDuringResim; }
		void SetEnabledDuringResim(bool bEnabled) { bEnabledDuringResim = bEnabled; }

		virtual void SetEnabled(bool InEnabled) override { bDisabled = !InEnabled; }
		virtual bool IsEnabled() const { return !bDisabled; }

		/// Settings is only modifiable on the game thread via FCharacterGroundConstraint
		/// but Data can be modified on the physics thread and is synced with the game thread
		const FCharacterGroundConstraintSettings& GetSettings() const { return Settings; }
		const FCharacterGroundConstraintDynamicData& GetData() const { return Data; }

		void SetData(const FCharacterGroundConstraintDynamicData& InData)
		{
			if (!CharacterParticle || bDisabled)
			{
				return;
			}

			Data = InData;

			FVec3 NewLocalCharacterPosition = ComputeLocalCharacterPosition();

			// If the movement target is close to zero and the character position has not
			// changed by much then it gets clamped to zero and we recompute the delta
			// position based on the previous target to avoid drift
			const float InputMovementThresholdSq = CVars::Chaos_CharacterGroundConstraint_InputMovementThreshold * CVars::Chaos_CharacterGroundConstraint_InputMovementThreshold;
			const float MovementThresholdSq = CVars::Chaos_CharacterGroundConstraint_ExternalMovementThreshold * CVars::Chaos_CharacterGroundConstraint_ExternalMovementThreshold;
			if ((InData.TargetDeltaPosition.SizeSquared() > InputMovementThresholdSq) || ((NewLocalCharacterPosition - LocalCharacterPosition).SizeSquared() > MovementThresholdSq))
			{
				LocalCharacterPosition = NewLocalCharacterPosition;
			}
			else
			{
				if (GroundParticle)
				{
					Data.TargetDeltaPosition = GroundParticle->GetR() * LocalCharacterPosition + GroundParticle->GetX() - CharacterParticle->GetX() + Data.TargetDeltaPosition;
				}
				else
				{
					Data.TargetDeltaPosition = LocalCharacterPosition - CharacterParticle->GetX() + Data.TargetDeltaPosition;
				}
			}

			SetDirtyFlag(EDirtyDataFlags::Data);
		}

		void SetSettings(const FCharacterGroundConstraintSettings& InSettings)
		{
			Settings = InSettings;
			SetDirtyFlag(EDirtyDataFlags::Settings);
		}

		FCharacterGroundConstraintSettings& GetSettings_Mutable()
		{
			SetDirtyFlag(EDirtyDataFlags::Settings);
			return Settings;
		}

		// Declared final so that TPBDConstraintGraphRuleImpl::AddToGraph() does not need to hit vtable
		virtual FParticlePair GetConstrainedParticles() const override final { return { CharacterParticle, GroundParticle }; }
		FGeometryParticleHandle* GetCharacterParticle() const { return CharacterParticle; }
		FGeometryParticleHandle* GetGroundParticle() const { return GroundParticle; }

		void SetGroundParticle(FGeometryParticleHandle* InGroundParticle)
		{
			if (InGroundParticle != GroundParticle)
			{
				if (InGroundParticle)
				{
					if (FPBDRigidParticleHandle* PBDRigid = InGroundParticle->CastToRigidParticle())
					{
						if (PBDRigid->Disabled())
						{
							InGroundParticle = nullptr;
						}
					}
				}

				if (GroundParticle)
				{
					GroundParticle->RemoveConstraintHandle(this);
				}
				GroundParticle = InGroundParticle;
				if (GroundParticle)
				{
					GroundParticle->AddConstraintHandle(this);
				}
				LocalCharacterPosition = ComputeLocalCharacterPosition();

				SetDirtyFlag(EDirtyDataFlags::GroundParticle);
			}
		}

		// Get the force applied by the solver due to this constraint. Units are ML/T^2
		FVec3 GetSolverAppliedForce() const { return SolverAppliedForce; }

		// Get the torque applied by the solver due to this constraint. Units are ML^2/T^2
		FVec3 GetSolverAppliedTorque() const { return SolverAppliedTorque; }

		bool HasGroundParticleChanged() const
		{
			return IsFlagSet(EDirtyDataFlags::GroundParticle);
		}

		bool HaveSettingsChanged() const
		{
			return IsFlagSet(EDirtyDataFlags::Settings);
		}

		bool HasDataChanged() const
		{
			return IsFlagSet(EDirtyDataFlags::Data);
		}

		void ClearDirtyFlags()
		{
			DirtyFlags = 0;
		}

	private:
		friend class FCharacterGroundConstraintContainer; // For creation
		friend class FCharacterGroundConstraintProxy; // For setting the data from the game thread constraint
		friend class Private::FCharacterGroundConstraintContainerSolver; // For setting the solver force and torque
		friend class ChaosTest::CharacterGroundConstraintContainerTest; // For testing internals

		enum class EDirtyDataFlags : uint8
		{
			GroundParticle	= 1 << 0,
			Settings		= 1 << 1,
			Data			= 1 << 2
		};
		uint8 DirtyFlags;
		bool IsFlagSet(EDirtyDataFlags Flag) const 
		{
			return (DirtyFlags & (uint8)Flag) == (uint8)Flag;
		}
		void SetDirtyFlag(EDirtyDataFlags Flag)
		{
			DirtyFlags |= (uint8)Flag;
		}

		FVec3 ComputeLocalCharacterPosition()
		{
			if (!CharacterParticle || bDisabled)
			{
				return FVec3::ZeroVector;
			}

			FVec3 LocalPos = CharacterParticle->GetX();

			if (GroundParticle)
			{
				LocalPos = GroundParticle->GetR().Inverse() * (LocalPos - GroundParticle->GetX());
			}

			return LocalPos;
		}

		FCharacterGroundConstraintSettings Settings;
		FCharacterGroundConstraintDynamicData Data;
		FVec3 SolverAppliedForce = FVec3::ZeroVector;
		FVec3 SolverAppliedTorque = FVec3::ZeroVector;
		FVec3 LocalCharacterPosition = FVec3::ZeroVector;
		FGeometryParticleHandle* CharacterParticle;
		FGeometryParticleHandle* GroundParticle;
		bool bDisabled = false;
		bool bEnabledDuringResim;
		EResimType ResimType = EResimType::FullResim;
		ESyncState SyncState = ESyncState::InSync;
	};

	/// Container class for all character ground constraints on the physics thread
	class FCharacterGroundConstraintContainer : public FPBDConstraintContainer
	{
	public:
		using Base = FPBDConstraintContainer;
		using FConstraints = TArrayView<FCharacterGroundConstraintHandle* const>;
		using FConstConstraints = TArrayView<const FCharacterGroundConstraintHandle* const>;

		CHAOS_API FCharacterGroundConstraintContainer();
		CHAOS_API virtual ~FCharacterGroundConstraintContainer();

		int32 NumConstraints() const { return Constraints.Num(); }

		CHAOS_API FCharacterGroundConstraintHandle* AddConstraint(
			const FCharacterGroundConstraintSettings& InConstraintSettings,
			const FCharacterGroundConstraintDynamicData& InConstraintData,
			FGeometryParticleHandle* CharacterParticle,
			FGeometryParticleHandle* GroundParticle = nullptr);

		CHAOS_API void RemoveConstraint(FCharacterGroundConstraintHandle* Constraint);

		FConstraints GetConstraints() { return MakeArrayView(Constraints); }
		FConstConstraints GetConstConstraints() const { return MakeArrayView(Constraints); }

		FCharacterGroundConstraintHandle* GetConstraint(int32 ConstraintIndex) { check(ConstraintIndex < NumConstraints()); return Constraints[ConstraintIndex]; }
		const FCharacterGroundConstraintHandle* GetConstraint(int32 ConstraintIndex) const { check(ConstraintIndex < NumConstraints()); return Constraints[ConstraintIndex]; }

		//////////////////////////////////////////////////////////////////////////
		// FConstraintContainer Implementation
		CHAOS_API virtual TUniquePtr<FConstraintContainerSolver> CreateSceneSolver(const int32 Priority) override final;
		CHAOS_API virtual TUniquePtr<FConstraintContainerSolver> CreateGroupSolver(const int32 Priority) override final;
		virtual int32 GetNumConstraints() const override final { return Constraints.Num(); }
		virtual void ResetConstraints() override final {}
		CHAOS_API virtual void AddConstraintsToGraph(Private::FPBDIslandManager& IslandManager) override final;
		CHAOS_API virtual void PrepareTick() override final;
		CHAOS_API virtual void UnprepareTick() override final;
		CHAOS_API virtual void DisconnectConstraints(const TSet<TGeometryParticleHandle<FReal, 3>*>& RemovedParticles) override final;
		CHAOS_API virtual void OnDisableParticle(FGeometryParticleHandle* DisabledParticle) override final;
		CHAOS_API virtual void OnEnableParticle(FGeometryParticleHandle* EnabledParticle) override final;

		//////////////////////////////////////////////////////////////////////////
		// Required API from FConstraintContainer
		CHAOS_API void SetConstraintEnabled(int32 ConstraintIndex, bool bEnabled);
		bool IsConstraintEnabled(int32 ConstraintIndex) const { check(ConstraintIndex < NumConstraints()); return Constraints[ConstraintIndex]->IsEnabled(); }

	private:
		CHAOS_API bool CanEvaluate(const FCharacterGroundConstraintHandle* Constraint) const;

		TObjectPool<FCharacterGroundConstraintHandle> ConstraintPool;
		TArray<FCharacterGroundConstraintHandle*> Constraints;
	};
}
