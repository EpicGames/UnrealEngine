// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"

namespace Chaos
{
	class FCharacterGroundConstraintSettings
	{
	public:
		bool operator==(const FCharacterGroundConstraintSettings& Other) const
		{
			return !FMemory::Memcmp(this, &Other, sizeof(*this));
		}

		FVec3 VerticalAxis = FVec3(0.0, 0.0, 1.0);	/// World space up direction (default z axis)
		FReal TargetHeight = 0.0;					/// Desired distance from the character body to the ground
		FReal RadialForceLimit = 1500.0;			/// How much force the character can apply parallel to the ground plane to reach the target position
		FReal FrictionForceLimit = 100.0;			/// How much force the character can apply parallel to the ground plane to reach the target position when standing on an unwalkable incline
		FReal TwistTorqueLimit = 1000.0;			///	How much torque the character can apply about the vertical axis to reach the target facing direction
		FReal SwingTorqueLimit = 5000.0;			///	How much torque the character can apply about the other axes to remain upright
		FReal CosMaxWalkableSlopeAngle = 0.633;		/// Cosine of the maximum angle in degrees that the character is allowed to walk on
		FReal DampingFactor = 0.0;					/// Applies a damping to the vertical ground constraint making it softer. Units: /T
		FReal AssumedOnGroundHeight = 2.0;			/// Below this height the character is assumed to be on the ground and can apply force/torque to reach the target position and facing
		FReal MotionTargetMassBias = 1.0;			/// Scaling factor applied to reduce the mass of the ground body in the linear motion target constraint
		FReal RadialForceMotionTargetScaling = 1.0; /// Scale the radial force to always reach this fraction of the motion target
		FReal MaxCharacterGroundMassRatio = -1.0f;	/// Maximum ratio for the mass of character mass to ground mass. A negative value indicates no mass conditioning is applied
		void* UserData = nullptr;
	};

	class FCharacterGroundConstraintDynamicData
	{
	public:
		bool operator==(const FCharacterGroundConstraintDynamicData& Other) const
		{
			return !FMemory::Memcmp(this, &Other, sizeof(*this));
		}

		FVec3 GroundNormal = FVec3::ZAxisVector;		/// World space ground normal
		FVec3 TargetDeltaPosition = FVec3::ZeroVector;	///	Target linear movement vector. Will be projected onto ground plane
		FReal TargetDeltaFacing = 0.0;					/// Target rotation in radians about the vertical axis
		FReal GroundDistance = 1.0e10f;					/// Distance from the character body to the ground
		FReal CosMaxWalkableSlopeAngle = 0.633;			/// Override for max walkable slope angle
	};

} // namespace Chaos
