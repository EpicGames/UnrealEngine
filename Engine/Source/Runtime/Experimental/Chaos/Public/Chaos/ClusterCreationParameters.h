// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/ParticleHandle.h"
#include "Chaos/ParticleHandleFwd.h"
#include "Chaos/Defines.h"

namespace Chaos
{
	class FBVHParticles;

	struct FClusterCreationParameters
	{
		enum EConnectionMethod
		{
			PointImplicit = 0,
			DelaunayTriangulation,
			MinimalSpanningSubsetDelaunayTriangulation,
			PointImplicitAugmentedWithMinimalDelaunay,
			BoundsOverlapFilteredDelaunayTriangulation,
			None
		};

		FClusterCreationParameters(
			FReal CoillisionThicknessPercentIn = (FReal)0.3
			, int32 MaxNumConnectionsIn = 100
			, bool bCleanCollisionParticlesIn = true
			, bool bCopyCollisionParticlesIn = true
			, bool bGenerateConnectionGraphIn = true
			, EConnectionMethod ConnectionMethodIn = EConnectionMethod::MinimalSpanningSubsetDelaunayTriangulation
			, FReal ConnectionGraphBoundsFilteringMarginIn = 0
			, FBVHParticles* CollisionParticlesIn = nullptr
			, Chaos::TPBDRigidClusteredParticleHandle<Chaos::FReal,3>* ClusterParticleHandleIn = nullptr
			, const FVec3& ScaleIn = FVec3::OneVector
			, bool bIsAnchoredIn = false
			, bool bInEnableStrainOnCollision = true
		)
			: CoillisionThicknessPercent(CoillisionThicknessPercentIn)
			, MaxNumConnections(MaxNumConnectionsIn)
			, bCleanCollisionParticles(bCleanCollisionParticlesIn)
			, bCopyCollisionParticles(bCopyCollisionParticlesIn)
			, bGenerateConnectionGraph(bGenerateConnectionGraphIn)
			, ConnectionMethod(ConnectionMethodIn)
			, ConnectionGraphBoundsFilteringMargin(ConnectionGraphBoundsFilteringMarginIn)
			, CollisionParticles(CollisionParticlesIn)
			, ClusterParticleHandle(ClusterParticleHandleIn)
			, Scale(ScaleIn)
			, bIsAnchored(bIsAnchoredIn)
			, bEnableStrainOnCollision(bInEnableStrainOnCollision)
		{}

		FReal CoillisionThicknessPercent;
		int32 MaxNumConnections;
		bool bCleanCollisionParticles;
		bool bCopyCollisionParticles;
		bool bGenerateConnectionGraph;
		EConnectionMethod ConnectionMethod;
		FReal ConnectionGraphBoundsFilteringMargin;
		FBVHParticles* CollisionParticles;
		Chaos::FPBDRigidClusteredParticleHandle* ClusterParticleHandle;
		FVec3 Scale;
		bool bIsAnchored;
		bool bEnableStrainOnCollision;
	};
}
