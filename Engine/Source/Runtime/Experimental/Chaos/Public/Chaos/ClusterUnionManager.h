// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/ClusterCreationParameters.h"
#include "Chaos/Framework/ArrayAlgorithm.h"
#include "Chaos/ParticleHandleFwd.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Misc/EnumClassFlags.h"

namespace Chaos
{
	using FClusterUnionIndex = int32;
	using FClusterUnionExplicitIndex = int32;

	class FRigidClustering;
	class FPBDRigidsEvolutionGBF;
	struct FClusterCreationParameters;

	enum class EClusterUnionOperation
	{
		Add,
		// AddReleased is the original behavior where if the particle to be added is a cluster, we will release the cluster first
		// and add its children instead.
		AddReleased,
		Remove,
		UpdateChildToParent
	};

	enum class EClusterUnionOperationTiming
	{
		Never,
		Defer,
		Immediate
	};

	enum class EUpdateClusterUnionPropertiesFlags : int32
	{
		None = 0,
		RecomputeMassOrientation = 1 << 0,
		ForceGenerateConnectionGraph = 1 << 1,
		IncrementalGenerateConnectionGraph = 1 << 2,
		UpdateKinematicProperties = 1 << 3,
		ForceGenerateGeometry = 1 << 4,
		IncrementalGenerateGeometry = 1 << 5,
		ConnectivityCheck = 1 << 6,
		All = RecomputeMassOrientation | ForceGenerateConnectionGraph | UpdateKinematicProperties | ForceGenerateGeometry | ConnectivityCheck
	};
	ENUM_CLASS_FLAGS(EUpdateClusterUnionPropertiesFlags);

	enum class EClusterUnionConnectivityOperation : int8
	{
		Add,
		Remove
	};

	enum class EClusterUnionGeometryOperation : int8
	{
		Add,
		Remove,
		Refresh
	};

	struct FClusterUnionCreationParameters
	{
		FClusterUnionExplicitIndex ExplicitIndex = INDEX_NONE;
		const FUniqueIdx* UniqueIndex = nullptr;
		uint32 ActorId = 0;
		uint32 ComponentId = 0;
		int32 GravityGroupOverride = INDEX_NONE;
	};

	struct FClusterUnionParticleProperties
	{
		FClusterUnionParticleProperties()
			: bIsAuxiliaryParticle(0)
			, bEdgesAreGenerated(0)
		{}
		// An auxiliary particle will be removed from the cluster union if FRigidClustering::HandleConnectivityOnReleaseClusterParticle detects an island made up of only auxiliary particles.
		uint8 bIsAuxiliaryParticle: 1;
		// Really just a protection for external users accidentally incurring a perf cost if they try to manually generate edges. We will never do this more than once per particle per cluster union.
		uint8 bEdgesAreGenerated: 1;
	};

	struct FClusterUnion
	{
		// The root cluster particle that we created internally to represent the cluster.
		FPBDRigidClusteredParticleHandle* InternalCluster;

		// The thread-safe collision geometry that can be shared between the GT and PT.
		Chaos::FImplicitObjectPtr Geometry;

		// All the particles that belong to this cluster.
		TArray<FPBDRigidParticleHandle*> ChildParticles;

		// All the particles that belong to this cluster that are currently in the implicit object geometry. There is a delay between removal from ChildParticles and its subsequent
		// removal from GeometryChildParticles.
		TArray<FPBDRigidParticleHandle*> GeometryChildParticles;

		// Additional properties that are only relevant for particles that have been added into a cluster union.
		TMap<FPBDRigidParticleHandle*, FClusterUnionParticleProperties> ChildProperties;

		// The internal index used to reference the cluster union.
		FClusterUnionIndex InternalIndex;

		// An explicit index set by the user if any.
		FClusterUnionExplicitIndex ExplicitIndex;

		// Need to remember the parameters used to create the cluster so we can update it later.
		FClusterCreationParameters Parameters;

		// Other parameters that aren't related to clusters generally but are related to info we need about the cluster union.
		FClusterUnionCreationParameters ClusterUnionParameters;

		// Whether or not the position/rotation needs to be computed the first time a particle is added.
		bool bNeedsXRInitialization : 1 = true;

		// Whether or not the anchor got set by the GT and we shouldn't try to recompute its value.
		bool bAnchorLock : 1 = false;

		// Whether or not we need to check for connectivity for this cluster union.
		bool bCheckConnectivity : 1 = false;

		// Whether or not to generate connectivity edges (used to differentiate between server graphs vs client graphs).
		bool bGenerateConnectivityEdges : 1 = true;

		// Whether the geometry for the union has change such that we will need to push a copy to the game thread
		bool bGeometryModified : 1 = true;

		// Pending particles that need to be added into the connectivity graph.
		TArray<TPair<FPBDRigidParticleHandle*, EClusterUnionConnectivityOperation>> PendingConnectivityOperations;

		bool IsGravityOverrideSet() const { return ClusterUnionParameters.GravityGroupOverride != INDEX_NONE; }

		const TArray<FPBDRigidParticleHandle*>& GetPendingGeometryOperationParticles(EClusterUnionGeometryOperation Op) const;
		void AddPendingGeometryOperation(EClusterUnionGeometryOperation Op, FPBDRigidParticleHandle* Particle);
		void ClearAllPendingGeometryOperations();
		void ClearPendingGeometryOperations(EClusterUnionGeometryOperation Op);

	private:
		// Pending particles that need to be added into/removed from the cluster union's geometry.
		// These need to be arrays to be able to maintain ordering.
		TMap<EClusterUnionGeometryOperation, TArray<FPBDRigidParticleHandle*>> PendingGeometryOperations;
	};

	struct FClusterUnionChildToParentUpdate
	{
		FTransform ChildToParent;
		bool bLock = false;
		FClusterUnionIndex ClusterUnionIndex = INDEX_NONE;
	};

	/**
	 * This class is used by Chaos to create internal clusters that will
	 * cause one or more clusters to simulate together as a single rigid
	 * particle.
	 */
	class FClusterUnionManager
	{
	public:

		template<EThreadContext Id>
		static FImplicitObjectTransformed* CreateTransformGeometryForClusterUnion(TThreadRigidParticle<Id>* Child, const FTransform& Frame)
		{
			FImplicitObjectTransformed* TransformedChildGeometry = new TImplicitObjectTransformed<FReal, 3>(Child->GetGeometry(), Frame);
			TransformedChildGeometry->SetGeometry(Child->GetGeometry());
			return TransformedChildGeometry;
		}

		CHAOS_API FClusterUnionManager(FRigidClustering& InClustering, FPBDRigidsEvolutionGBF& InEvolution);

		// Creates a new cluster union with an automatically assigned cluster union index.
		CHAOS_API FClusterUnionIndex CreateNewClusterUnion(const FClusterCreationParameters& Parameters, const FClusterUnionCreationParameters& ClusterUnionParameters = FClusterUnionCreationParameters{});

		// Destroy a given cluster union.
		CHAOS_API void DestroyClusterUnion(FClusterUnionIndex Index);

		// Add a new operation to the queue. Note that we currently only support the pending/flush operations for explicit operations. The behavior is legacy anyway so this should be fine.
		CHAOS_API void AddPendingExplicitIndexOperation(FClusterUnionExplicitIndex Index, EClusterUnionOperation Op, const TArray<FPBDRigidParticleHandle*>& Particles);
		CHAOS_API void AddPendingClusterIndexOperation(FClusterUnionIndex Index, EClusterUnionOperation Op, const TArray<FPBDRigidParticleHandle*>& Particles);

		// Actually performs the change specified in the FClusterUnionOperationData structure.
		CHAOS_API void HandleAddOperation(FClusterUnionIndex ClusterIndex, const TArray<FPBDRigidParticleHandle*>& InParticles, bool bReleaseClustersFirst);

		// Removes the specified particles from the specified cluster.
		CHAOS_API void HandleRemoveOperation(FClusterUnionIndex ClusterIndex, const TArray<FPBDRigidParticleHandle*>& Particles, EClusterUnionOperationTiming UpdateClusterPropertiesTiming);

		// Helper function to remove particles given only the particle handle. This will consult the lookup table to find which cluster the particle is in before doing a normal remove operation.
		CHAOS_API void HandleRemoveOperationWithClusterLookup(const TArray<FPBDRigidParticleHandle*>& InParticles, EClusterUnionOperationTiming UpdateClusterPropertiesTiming);

		// Performs the queued child to parent modifications.
		CHAOS_API void HandleUpdateChildToParentOperation(FClusterUnionIndex ClusterIndex, const TArray<FPBDRigidParticleHandle*>& Particles);

		// Will be called at the beginning of every time step to ensure that all the expected cluster unions have been modified.
		CHAOS_API void FlushPendingOperations();

		// Update cluster union properties here if they were deferred. This can be called manually but will otherwise also be handled in FlushPendingOperations.
		CHAOS_API void HandleDeferredClusterUnionUpdateProperties();
		
		// Access the cluster union externally.
		CHAOS_API FClusterUnion* FindClusterUnionFromExplicitIndex(FClusterUnionExplicitIndex Index);
		CHAOS_API FClusterUnion* FindClusterUnion(FClusterUnionIndex Index);
		CHAOS_API const FClusterUnion* FindClusterUnion(FClusterUnionIndex Index) const;
		CHAOS_API FClusterUnion* FindClusterUnionFromParticle(FPBDRigidParticleHandle* Particle);
		CHAOS_API const FClusterUnion* FindClusterUnionFromParticle(const FPBDRigidParticleHandle* Particle) const;
		CHAOS_API FClusterUnionIndex FindClusterUnionIndexFromParticle(const FPBDRigidParticleHandle* Particle) const;

		// An extension to FindClusterUnionIndexFromParticle to check whether or not the given particle is the cluster union particle itself.
		CHAOS_API bool IsClusterUnionParticle(FPBDRigidClusteredParticleHandle* Particle);

		// Changes the ChildToParent of a number of particles in a cluster union. bLock will prevent other functions from overriding the ChildToParent (aside from another call to UpdateClusterUnionParticlesChildToParent).
		CHAOS_API void UpdateClusterUnionParticlesChildToParent(FClusterUnionIndex Index, const TArray<FPBDRigidParticleHandle*>& Particles, const TArray<FTransform>& ChildToParent, bool bLock);

		// Update the cluster union's properties after its set of particle changes.
		CHAOS_API void UpdateAllClusterUnionProperties(FClusterUnion& ClusterUnion, EUpdateClusterUnionPropertiesFlags Flags = EUpdateClusterUnionPropertiesFlags::All);

		CHAOS_API void AddParticleToConnectionGraphInCluster(FClusterUnion& ClusterUnion, FPBDRigidParticleHandle* Particle);

		CHAOS_API void GenerateInterclusterEdgesForParticle(FClusterUnion& ClusterUnion, FPBDRigidParticleHandle* Particle);
		CHAOS_API void GenerateInterclusterEdgesBetweenParticles(FClusterUnion& ClusterUnion, FPBDRigidParticleHandle* Particle, FPBDRigidParticleHandle* OtherParticle);

		// Returns all cluster unions. Really meant only to be used for debugging.
		const TMap<FClusterUnionIndex, FClusterUnion>& GetAllClusterUnions() const { return ClusterUnions; }

		bool IsDirectlyConnectedToMainParticleInClusterUnion(const FClusterUnion& ClusterUnion, FPBDRigidParticleHandle* Particle) const;

		// Put in a deferred request to update the properties of the specified cluster union. Flags can limit the scope of the update.
		CHAOS_API void RequestDeferredClusterPropertiesUpdate(FClusterUnionIndex ClusterIndex, EUpdateClusterUnionPropertiesFlags Flags);
	private:
		FRigidClustering& MClustering;
		FPBDRigidsEvolutionGBF& MEvolution;

		using FClusterOpMap = TMap<EClusterUnionOperation, TArray<FPBDRigidParticleHandle*>>;
		template<typename TIndex>
		using TClusterIndexOpMap = TMap<TIndex, FClusterOpMap>;

		TClusterIndexOpMap<FClusterUnionIndex> PendingClusterIndexOperations;
		TClusterIndexOpMap<FClusterUnionExplicitIndex> PendingExplicitIndexOperations;
		TMap<FPBDRigidClusteredParticleHandle*, FClusterUnionChildToParentUpdate> PendingChildToParentUpdates;
		TSet<FPBDRigidClusteredParticleHandle*> PendingParticlesToUndoChildToParentLock;

		template<typename TIndex>
		void AddPendingOperation(TClusterIndexOpMap<TIndex>& OpMap, TIndex Index, EClusterUnionOperation Op, const TArray<FPBDRigidParticleHandle*>& Particles)
		{
			FClusterOpMap& Ops = OpMap.FindOrAdd(Index);
			TArray<FPBDRigidParticleHandle*>& OpData = Ops.FindOrAdd(Op);
			OpData.Append(Particles);
		}

		// All of our actively managed cluster unions. We need to keep track of these
		// so a user could use the index to request modifications to a specific cluster union.
		TMap<FClusterUnionIndex, FClusterUnion> ClusterUnions;

		// The set of cluster unions that we want to defer updating their cluster union properties for.
		TMap<FClusterUnionIndex, EUpdateClusterUnionPropertiesFlags> DeferredClusterUnionsForUpdateProperties;
		
		//
		// There are two ways we can pick a new union index:
		// - If a cluster union gets released/destroyed, that index can be reused.
		// - Otherwise, we use the NextAvailableUnionIndex which is just the max index we've seen + 1.
		//
		CHAOS_API FClusterUnionIndex ClaimNextUnionIndex();
		TArray<FClusterUnionIndex> ReusableIndices;
		FClusterUnionIndex NextAvailableUnionIndex = 1;

		// Using the user's passed in FClusterUnionIndex may result in strange unexpected behavior if the 
		// user creates a cluster with a specified index. Thus we will map all explicitly requested indices
		// (i.e. an index that comes in via FClusterUnionOperationData for the first time) to an automatically
		// generated index (i.e one that would returned via CreateNewClusterUnion).
		TMap<FClusterUnionExplicitIndex, FClusterUnionIndex> ExplicitIndexMap;
		CHAOS_API FClusterCreationParameters DefaultClusterCreationParameters() const;

		// If no cluster index is set but an explicit index is set, map the explicit index to a regular index.
		CHAOS_API FClusterUnionIndex GetOrCreateClusterUnionIndexFromExplicitIndex(FClusterUnionExplicitIndex InIndex);

		// Forcefully recreate the shared geometry on a cluster. Potentially expensive so ideally should be used rarely.
		CHAOS_API FImplicitObjectPtr ForceRecreateClusterUnionGeometry(const FClusterUnion& Union);

		UE_DEPRECATED(5.4, "Please use ForceRecreateClusterUnionGeometry instead")
		TSharedPtr<FImplicitObject, ESPMode::ThreadSafe> ForceRecreateClusterUnionSharedGeometry(const FClusterUnion& Union)
		{
			check(false);
			return nullptr;
		}

		// Handles updating the cluster union.
		CHAOS_API void DeferredClusterUnionUpdate(FClusterUnion& Union, EUpdateClusterUnionPropertiesFlags Flags);

		// Flush the cluster union's incremental connectivity operations
		CHAOS_API void FlushIncrementalConnectivityGraphOperations(FClusterUnion& ClusterUnion);

		// Forcefully regenerate the cluster union's geometry.
		void ForceRegenerateGeometry(FClusterUnion& ClusterUnion, const TSet<FPBDRigidParticleHandle*>& FullChildrenSet);

		// Flush the cluster union's incremental geometry operations.
		void FlushIncrementalGeometryOperations(FClusterUnion& ClusterUnion);
	};

	template<typename TParticle>
	void TransferClusterUnionShapeData(const TUniquePtr<Chaos::FPerShapeData>& ShapeData, TParticle* TemplateParticle, const TUniquePtr<Chaos::FPerShapeData>& TemplateShape, int32 ActorId, int32 ComponentId)
	{
		if (ShapeData && TemplateShape)
		{
			{
				FCollisionData Data = TemplateShape->GetCollisionData();
				Data.UserData = nullptr;
				ShapeData->SetCollisionData(Data);
			}

			{
				const Chaos::Filter::FInstanceData InstanceData(ActorId, ComponentId);
				ShapeData->SetFilterInstanceData(InstanceData);
				ShapeData->SetShapeFilterData(TemplateShape->GetShapeFilterData());
			}

			{
				const int32 NumTemplateMaterials = TemplateShape->NumMaterials();

				if(NumTemplateMaterials == 1)
				{
					// We have a special case for 1 material, calling GetMaterials
					// would allocate a material storage object to apply to the shape
					// whereas for 1 material we store it inline to the shape data
					ShapeData->SetMaterial(TemplateShape->GetMaterial(0));
				}
				else if(NumTemplateMaterials > 1)
				{
					ShapeData->SetMaterials(TemplateShape->GetMaterials());
				}
			}
		}
	}

	/**
	 * Wraps any additions to the cluster union geometry. We assume that the input lambda Func
	 * will modify the cluster union particle's geometry *and* shapes with the input particles.
	 * This function will make sure the newly added shapes are setup properly with the expected
     * properties (sim data, query data, user data etc.).
	 */
	template<typename TClusterParticle, typename TParticle, typename TLambda>
	void ModifyAdditionOfChildrenToClusterUnionGeometry(TClusterParticle* ClusterParticle, const TArray<TParticle*>& Particles, int32 ActorId, int32 ComponentId, TLambda&& Func)
	{
		if (!ClusterParticle || Particles.IsEmpty())
		{
			return;
		}

		FImplicitObjectUnion* ImplicitUnion = ClusterParticle->GetGeometry() ? ClusterParticle->GetGeometry()->template AsA<FImplicitObjectUnion>() : nullptr;
		const int32 OldNumChildShapes = ImplicitUnion ? ImplicitUnion->GetNumRootObjects() : 0;

		Func();

		// We must have a union at this point since the contract is that Func is adding a shape
		// which means we must be in a union already.
		check(ClusterParticle->GetGeometry() != nullptr);
		ImplicitUnion = ClusterParticle->GetGeometry()->template AsA<FImplicitObjectUnion>();
		check(ImplicitUnion != nullptr);

		const int32 NewNumChildShapes = ImplicitUnion->GetNumRootObjects();
		const FShapesArray& ShapeArray = ClusterParticle->ShapesArray();
		check(ShapeArray.Num() == NewNumChildShapes);
		check(Particles.Num() == NewNumChildShapes - OldNumChildShapes);
	
		for (int32 Index = 0; Index < Particles.Num(); ++Index)
		{
			TParticle* Particle = Particles[Index];
			if (!Particle)
			{
				continue;
			}

			const TUniquePtr<Chaos::FPerShapeData>& TemplateShape = Particle->ShapesArray()[0];
			if (!TemplateShape)
			{
				continue;
			}

			const int32 ShapeIndex =  OldNumChildShapes + Index;
			if (const TUniquePtr<Chaos::FPerShapeData>& ShapeData = ShapeArray[ShapeIndex])
			{
				TransferClusterUnionShapeData(ShapeData, Particle, TemplateShape, ActorId, ComponentId);
			}
		}
	}

	/**
	 * Utility function to remove particles from a cluster union's geometry incrementally.
	 */
	template<typename TClusterParticle, typename TParticle>
	void RemoveParticlesFromClusterUnionGeometry(TClusterParticle* ClusterParticle, const TArray<TParticle*>& ShapeParticles, TArray<TParticle*>& AllChildParticles)
	{
		if (!ensure(ClusterParticle))
		{
			return;
		}

		// Don't try to remove anything if the shape array is already empty 
		if (!ClusterParticle->ShapesArray().IsEmpty() && !AllChildParticles.IsEmpty())
		{
#if WITH_EDITOR
			ensureMsgf(
				AllChildParticles.Num() == ClusterParticle->ShapesArray().Num(),
				TEXT("RemoveParticlesFromClusterUnionGeometry : More than one shape per child particle : ShapeArray=[%d] AllChildParticles=[%d] ShapeParticles=[%d]"),
				ClusterParticle->ShapesArray().Num(),
				AllChildParticles.Num(),
				ShapeParticles.Num()
			);
#else
			checkf(
				AllChildParticles.Num() == ClusterParticle->ShapesArray().Num(),
				TEXT("RemoveParticlesFromClusterUnionGeometry : More than one shape per child particle : ShapeArray=[%d] AllChildParticles=[%d] ShapeParticles=[%d]"),
				ClusterParticle->ShapesArray().Num(),
				AllChildParticles.Num(),
				ShapeParticles.Num()
			);
#endif
			

			TArray<int32> ShapeIndicesToRemove;
			ShapeIndicesToRemove.Reserve(ShapeParticles.Num());
			for (TParticle* ShapeParticle : ShapeParticles)
			{
				check(ShapeParticle != nullptr);
				const int32 Index = AllChildParticles.Find(ShapeParticle);
				if (Index != INDEX_NONE)
				{
					ShapeIndicesToRemove.Add(Index);
				}
			}

			ShapeIndicesToRemove.Sort();

			ClusterParticle->RemoveShapesAtSortedIndices(ShapeIndicesToRemove);

			RemoveArrayItemsAtSortedIndices(AllChildParticles, ShapeIndicesToRemove);

#if WITH_EDITOR
			ensureMsgf(
				AllChildParticles.Num() == ClusterParticle->ShapesArray().Num(),
				TEXT("RemoveParticlesFromClusterUnionGeometry : More than one shape per child particle : ShapeArray=[%d] AllChildParticles=[%d] ShapeParticles=[%d]"),
				ClusterParticle->ShapesArray().Num(),
				AllChildParticles.Num(),
				ShapeParticles.Num()
			);
#else
			checkf(
				AllChildParticles.Num() == ClusterParticle->ShapesArray().Num(),
				TEXT("RemoveParticlesFromClusterUnionGeometry : More than one shape per child particle : ShapeArray=[%d] AllChildParticles=[%d] ShapeParticles=[%d]"),
				ClusterParticle->ShapesArray().Num(),
				AllChildParticles.Num(),
				ShapeParticles.Num()
			);
#endif
		}
		// If we remove particles from the cluster union geometry then we need to switch the geometry back to a FImplicitObjectUnionClustered to avoid errors with empty unions.
		if (ClusterParticle->ShapesArray().IsEmpty())
		{
			ClusterParticle->SetGeometry(MakeImplicitObjectPtr<FImplicitObjectUnionClustered>());
		}
	}
}
