// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryCollection/Facades/CollectionPositionTargetFacade.h"

namespace Chaos::Softs
{
	/** Base class for non-zero length springs.*/
	class FEmbeddedSpringBaseFacade : public GeometryCollection::Facades::FPositionTargetFacade
	{
	public:
		// Group PositionTargets
		static CHAOS_API const FName SpringLength;
		static CHAOS_API const FName CompressionStiffness; // Base Stiffness is ExtensionStiffness

		// Collections of springs
		static CHAOS_API const FName SpringConstraintGroupName;
		static CHAOS_API const FName ConstraintStart;
		static CHAOS_API const FName ConstraintEnd;
		static CHAOS_API const FName ConstraintEndPointNumIndices;
		static CHAOS_API const FName ConstraintName;

		CHAOS_API FEmbeddedSpringBaseFacade(FManagedArrayCollection& InCollection, const FName& InVerticesGroup = FGeometryCollection::VerticesGroup);
		CHAOS_API FEmbeddedSpringBaseFacade(const FManagedArrayCollection& InCollection, const FName& InVerticesGroup = FGeometryCollection::VerticesGroup);

		CHAOS_API bool IsValid() const;
		CHAOS_API void DefineSchema();

	protected:
		//~ Group PositionTargets
		TManagedArrayAccessor<float> SpringLengthAttribute;
		TManagedArrayAccessor<float> CompressionStiffnessAttribute;

		//~ Spring Constraint Group
		TManagedArrayAccessor<int32> ConstraintStartAttribute;
		TManagedArrayAccessor<int32> ConstraintEndAttribute;
		TManagedArrayAccessor<FUintVector2> ConstraintEndPointNumIndicesAttribute;
		TManagedArrayAccessor<FString> ConstraintNameAttribute;
	};

	/** Facade for managing groups of springs with the same embedding type (e.g., vertex-vertex or barycentric point-vertex) */
	class FEmbeddedSpringConstraintFacade final : public FEmbeddedSpringBaseFacade
	{
		typedef FEmbeddedSpringBaseFacade Base;
	public:
		FEmbeddedSpringConstraintFacade() = delete;
		FEmbeddedSpringConstraintFacade(const FEmbeddedSpringConstraintFacade&) = delete;
		FEmbeddedSpringConstraintFacade(FEmbeddedSpringConstraintFacade&&) = default;
		FEmbeddedSpringConstraintFacade& operator=(const FEmbeddedSpringConstraintFacade&) = delete;
		FEmbeddedSpringConstraintFacade& operator=(FEmbeddedSpringConstraintFacade&&) = delete;

		CHAOS_API void Reset();

		int32 GetConstraintIndex() const
		{
			return ConstraintIndex;
		}

		CHAOS_API int32 GetNumSprings() const;
		CHAOS_API void RemoveSprings(const TArray<int32>& SortedSpringsToRemove);

		/** Initialize from a list of Vertex-Vertex springs.
		 *  Spring Indices are VerticesGroup indices.
		 *  Stiffness and Damping weights are expected have a value in [0.f, 1.f]. They will be set to constant 0.f if not supplied.
		*/
		CHAOS_API void Initialize(const TConstArrayView<FIntVector2>& EndPoints, const TConstArrayView<float>& InSpringLength,
			const TConstArrayView<float>& InExtensionStiffnesWeight = TConstArrayView<float>(), const TConstArrayView<float>& InCompressionStiffnessWeight = TConstArrayView<float>(), const TConstArrayView<float>& InDampingWeight = TConstArrayView<float>(), const FString& InConstraintName = TEXT(""));

		/** Append from a list of Vertex-Vertex springs. Only succeeds if existing EndPointNumIndices == (1,1)
		 *  Spring Indices are VerticesGroup indices.
		 *  Stiffness and Damping weights are expected have a value in [0.f, 1.f]. They will be set to constant 0.f if not supplied.
		*/
		CHAOS_API void Append(const TConstArrayView<FIntVector2>& EndPoints, const TConstArrayView<float>& InSpringLength,
			const TConstArrayView<float>& InExtensionStiffnesWeight = TConstArrayView<float>(), const TConstArrayView<float>& InCompressionStiffnessWeight = TConstArrayView<float>(), const TConstArrayView<float>& InDampingWeight = TConstArrayView<float>());

		/** Initialize from a list of springs. This is generic for any number of end point indices. If you have vertex-vertex constraints, use the Initialize method above which skips setting weights.
		 *  Source Indices and Weights arrays are expected to be exactly EndPointNumIndices[0] long.
		 *  Target Indices and Weights arrays are expected to be exactly EndPointNumIndices[1] long.
		 *  Indices and Weights will be truncated or filled in with zeros if they aren't the expected length.
		 *  Spring Indices are VerticesGroup indices.
		 *  Stiffness and Damping weights are expected have a value in [0.f, 1.f]. They will be set to constant 0.f if not supplied.
		*/
		CHAOS_API void Initialize(const FUintVector2& EndPointNumIndices, const TConstArrayView<TArray<int32>>& InSourceIndices, const TConstArrayView<TArray<float>>& InSourceWeights,
			const TConstArrayView<TArray<int32>>& InTargetIndices, const TConstArrayView<TArray<float>>& InTargetWeights,
			const TConstArrayView<float>& InSpringLength,
			const TConstArrayView<float>& InExtensionStiffnesWeight = TConstArrayView<float>(), const TConstArrayView<float>& InCompressionStiffnessWeight = TConstArrayView<float>(), const TConstArrayView<float>& InDampingWeight = TConstArrayView<float>(), const FString& InConstraintName = TEXT(""));

		/** Append from a list of springs. This is generic for any number of end point indices. If you have vertex-vertex constraints, use the Append method above which skips setting weights.
		 *  Source Indices and Weights arrays are expected to be exactly EndPointNumIndices[0] long.
		 *  Target Indices and Weights arrays are expected to be exactly EndPointNumIndices[1] long.
		 *  Indices and Weights will be truncated or filled in with zeros if they aren't the expected length.
		 *  Spring Indices are VerticesGroup indices.
		 *  Stiffness and Damping weights are expected have a value in [0.f, 1.f]. They will be set to constant 0.f if not supplied.
		*/
		CHAOS_API void Append(const TConstArrayView<TArray<int32>>& InSourceIndices, const TConstArrayView<TArray<float>>& InSourceWeights,
			const TConstArrayView<TArray<int32>>& InTargetIndices, const TConstArrayView<TArray<float>>& InTargetWeights,
			const TConstArrayView<float>& InSpringLength,
			const TConstArrayView<float>& InExtensionStiffnesWeight = TConstArrayView<float>(), const TConstArrayView<float>& InCompressionStiffnessWeight = TConstArrayView<float>(), const TConstArrayView<float>& InDampingWeight = TConstArrayView<float>());

		//~ Spring Constraint Group
		FUintVector2 GetConstraintEndPointNumIndices() const
		{
			return ConstraintEndPointNumIndicesAttribute[ConstraintIndex];
		}

		const FString& GetConstraintName() const
		{
			return ConstraintNameAttribute[ConstraintIndex];
		}

		void SetConstraintName(const FString& InName)
		{
			ConstraintNameAttribute.ModifyAt(ConstraintIndex, InName);
		}

		//~ Group PositionTargets
		CHAOS_API TArrayView<float> GetSpringLength();
		CHAOS_API TConstArrayView<float> GetSpringLengthConst() const;
		CHAOS_API TArrayView<float> GetExtensionStiffness();
		CHAOS_API TConstArrayView<float> GetExtensionStiffnessConst() const;
		CHAOS_API TArrayView<float> GetCompressionStiffness();
		CHAOS_API TConstArrayView<float> GetCompressionStiffnessConst() const;
		CHAOS_API TArrayView<float> GetDamping();
		CHAOS_API TConstArrayView<float> GetDampingConst() const;
		CHAOS_API TConstArrayView<TArray<int32>> GetSourceIndexConst() const;
		CHAOS_API TConstArrayView<TArray<float>> GetSourceWeightsConst() const;
		CHAOS_API TConstArrayView<TArray<int32>> GetTargetIndexConst() const;
		CHAOS_API TConstArrayView<TArray<float>> GetTargetWeightsConst() const;
		/* Set Indices and Weights for a single spring.
		 *  Source Indices and Weights arrays are expected to be exactly EndPointNumIndices[0] long.
		 *  Target Indices and Weights arrays are expected to be exactly EndPointNumIndices[1] long.
		 *  Indices and Weights will be truncated or filled in with zeros if they aren't the expected length.
		 *  Spring Indices are VerticesGroup indices.*/
		CHAOS_API void SetIndicesAndWeights(const int32 SpringIndex, const TConstArrayView<int32>& InSourceIndices, const TConstArrayView<float>& InSourceWeights,
			const TConstArrayView<int32>& InTargetIndices, const TConstArrayView<float>& InTargetWeights);

		/** Initialize from another constraint. */
		CHAOS_API void Initialize(const FEmbeddedSpringConstraintFacade& Other, const int32 VertexOffset);
		/** Append from another constraint. Must have same ConstraintEndPointNumIndices*/
		CHAOS_API void Append(const FEmbeddedSpringConstraintFacade& Other, const int32 VertexOffset);

		CHAOS_API uint32 CalculateTypeHash(uint32 PreviousHash = 0) const;

		/** Remove springs with invalid vertices. */
		CHAOS_API void CleanupAndCompactInvalidSprings();

	private:
		friend class FEmbeddedSpringFacade;
		FEmbeddedSpringConstraintFacade(FManagedArrayCollection& InCollection, const FName& InVerticesGroup, int32 InConstraintIndex);
		FEmbeddedSpringConstraintFacade(const FManagedArrayCollection& InCollection, const FName& InVerticesGroup, int32 InConstraintIndex);

		void SetDefaults();
		void SetNumSprings(int32 NumSprings);

		//~ Spring Constraint Group
		void SetConstraintEndPointNumIndices(const FUintVector2& NumIndices)
		{
			ConstraintEndPointNumIndicesAttribute.ModifyAt(ConstraintIndex, NumIndices);
		}

		//~ Group PositionTargets
		TArrayView<TArray<int32>> GetSourceIndex();
		TArrayView<TArray<float>> GetSourceWeights();
		TArrayView<TArray<int32>> GetTargetIndex();
		TArrayView<TArray<float>> GetTargetWeights();
		int32 ConstraintIndex;
	};

	/** Facade for managing all FEmbeddedSpringConstraintFacade within a collection */
	class FEmbeddedSpringFacade final : public FEmbeddedSpringBaseFacade
	{
		typedef FEmbeddedSpringBaseFacade Base;
	public:

		FEmbeddedSpringFacade(FManagedArrayCollection& InCollection, const FName& InVerticesGroup = FGeometryCollection::VerticesGroup)
			: Base(InCollection, InVerticesGroup)
		{
		}

		FEmbeddedSpringFacade(const FManagedArrayCollection& InCollection, const FName& InVerticesGroup = FGeometryCollection::VerticesGroup)
			: Base(InCollection, InVerticesGroup)
		{}

		/** Append contents of another facade to this one. */
		CHAOS_API void Append(const FEmbeddedSpringFacade& Other, int32 VertexOffset);

		/** Constraints (lists of springs) management */
		int32 GetNumSpringConstraints() const
		{
			return ConstraintStartAttribute.Num();
		}
		CHAOS_API void SetNumSpringConstraints(int32 Num);
		CHAOS_API FEmbeddedSpringConstraintFacade GetSpringConstraint(int32 ConstraintIndex);
		CHAOS_API FEmbeddedSpringConstraintFacade GetSpringConstraintConst(int32 ConstraintIndex) const;
		CHAOS_API int32 AddSpringConstraint();
		FEmbeddedSpringConstraintFacade AddGetSpringConstraint()
		{
			return GetSpringConstraint(AddSpringConstraint());
		}
		CHAOS_API void RemoveSpringConstraints(const TArray<int32>& SortedDeletionList);

		CHAOS_API uint32 CalculateTypeHash(uint32 PreviousHash = 0) const;

		/** Remove springs with invalid vertices.*/
		CHAOS_API void CleanupAndCompactInvalidSprings();
	};
}
