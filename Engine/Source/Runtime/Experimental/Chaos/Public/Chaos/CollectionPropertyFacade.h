// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"
#include "Containers/Map.h"
#include "Math/Vector.h"

struct FManagedArrayCollection;

namespace Chaos::Softs
{
	/** Property flags, whether properties are enabled, animatable, ...etc. */
	enum class ECollectionPropertyFlags : uint8
	{
		None = 0,
		Enabled = 1 << 0,  /** Whether this property is enabled(so that it doesn't have to be removed from the collection when not needed). */
		Animatable = 1 << 1,  /** Whether this property needs to be set at every frame. This flag is ignored when the Intrinsic flag is also set. */
		Legacy = 1 << 2,  /** Whether this property has been set by a legacy system predating the property collection. Can be useful for overriding/upgrading some properties post conversion. */
		Interpolable = 1 << 3,  /** Whether this property can be interpolated. Used to allow interpolation when merging float properties */
		Intrinsic = 1 << 4,  /** Whether this property is intrinsically built into the simulated object model, rather than only affecting the simulation itself (see Animatable in this case). Changing this property requires a re-construction of the simulated object model to be fully effective. */
		//~ Add new flags above this line
		StringDirty = 1 << 6,  /** Whether this property's string has changed and needs to be updated at the next frame. */
		Dirty = 1 << 7  /** Whether this property's value has changed and needs to be updated at the next frame. */
	};
	ENUM_CLASS_FLAGS(ECollectionPropertyFlags)

	/** Weighted types are all property types that can have a pair of low and high values to be associated with a weight map. */
	template<typename T> struct TIsWeightedType { static constexpr bool Value = false; };
	template<> struct TIsWeightedType<bool> { static constexpr bool Value = true; };
	template<> struct TIsWeightedType<int32> { static constexpr bool Value = true; };
	template<> struct TIsWeightedType<float> { static constexpr bool Value = true; };
	template<> struct TIsWeightedType<FVector3f> { static constexpr bool Value = true; };

	/** Helper to disambiguate new FName based keys from deprecated FString-based keys.*/
	template<typename T> struct TIsStringAndNameConvertibleType { static constexpr bool Value = false; };
	template<> struct TIsStringAndNameConvertibleType<FString> { static constexpr bool Value = true; };
	template<> struct TIsStringAndNameConvertibleType<const FString&> { static constexpr bool Value = true; };
	template<> struct TIsStringAndNameConvertibleType<const WIDECHAR*> { static constexpr bool Value = true; };
	template<> struct TIsStringAndNameConvertibleType<const ANSICHAR*> { static constexpr bool Value = true; };
	template<> struct TIsStringAndNameConvertibleType<const UTF8CHAR*> { static constexpr bool Value = true; };

	/**
	 * Defines common API for reading simulation properties data and metadata.
	 * This is mainly used for the cloth simulation properties to provide
	 * weighted values that works in conjunction with weight maps.
	 */
	class FCollectionPropertyConstFacade
	{
	public:
		CHAOS_API explicit FCollectionPropertyConstFacade(const TSharedPtr<const FManagedArrayCollection>& InManagedArrayCollection);
		CHAOS_API virtual ~FCollectionPropertyConstFacade();

		FCollectionPropertyConstFacade() = delete;
		FCollectionPropertyConstFacade(const FCollectionPropertyConstFacade&) = delete;
		FCollectionPropertyConstFacade& operator=(const FCollectionPropertyConstFacade&) = delete;

		FCollectionPropertyConstFacade(FCollectionPropertyConstFacade&&) = default;
		FCollectionPropertyConstFacade& operator=(FCollectionPropertyConstFacade&&) = default;

		/** Return whether the facade is defined on the collection. */
		CHAOS_API bool IsValid() const;

		/** Return whether this collection facade is dirty (does not rely on individual property flags). */
		CHAOS_API bool IsDirty() const;

		/** Return the number of properties in this collection. */
		int32 Num() const { return KeyNameIndices.Num(); }

		/** Return the property index for the specified key if it exists, or INDEX_NONE otherwise. */
		int32 GetKeyNameIndex(const FName& Key) const { const int32* const Index = KeyNameIndices.Find(Key); return Index ? *Index : INDEX_NONE; }
		UE_DEPRECATED(5.7, "Use GetKeyNameIndex instead")
		int32 GetKeyIndex(const FString& Key) const { const int32* const Index = KeyNameIndices.Find(FName(Key)); return Index ? *Index : INDEX_NONE; }

		/** Return the property name (key) for the specified index. */
		//~ Values access per index, fast, no check, index must be valid (0 <= KeyIndex < Num())
		const FName& GetKeyName(int32 KeyIndex) const { return GetValue<const FName&>(KeyIndex, KeyNameArray); }
		UE_DEPRECATED(5.7, "Use GetKeyName instead")
		FString GetKey(int32 KeyIndex) const { return GetValue<const FName&>(KeyIndex, KeyNameArray).ToString(); }

		template<typename T UE_REQUIRES(TIsWeightedType<T>::Value)>
		T GetLowValue(int32 KeyIndex) const { return GetValue<T>(KeyIndex, LowValueArray); }

		template<typename T UE_REQUIRES(TIsWeightedType<T>::Value)>
		T GetHighValue(int32 KeyIndex) const { return GetValue<T>(KeyIndex, HighValueArray); }

		template<typename T UE_REQUIRES(TIsWeightedType<T>::Value)>
		TPair<T, T> GetWeightedValue(int32 KeyIndex) const { return MakeTuple(GetLowValue<T>(KeyIndex), GetHighValue<T>(KeyIndex)); }

		FVector2f GetWeightedFloatValue(int32 KeyIndex) const { return FVector2f(GetLowValue<float>(KeyIndex), GetHighValue<float>(KeyIndex)); }

		template<typename T UE_REQUIRES(TIsWeightedType<T>::Value)>
		T GetValue(int32 KeyIndex) const { return GetLowValue<T>(KeyIndex); }

		const FString& GetStringValue(int32 KeyIndex) const { return GetValue<const FString&>(KeyIndex, StringValueArray); }

		ECollectionPropertyFlags GetFlags(int32 KeyIndex) const { return (ECollectionPropertyFlags)FlagsArray[KeyIndex]; }

		bool IsEnabled(int32 KeyIndex) const { return HasAnyFlags(KeyIndex, ECollectionPropertyFlags::Enabled); }
		bool IsAnimatable(int32 KeyIndex) const { return HasAnyFlags(KeyIndex, ECollectionPropertyFlags::Animatable); }
		bool IsLegacy(int32 KeyIndex) const { return HasAnyFlags(KeyIndex, ECollectionPropertyFlags::Legacy); }
		bool IsIntrinsic(int32 KeyIndex) const { return HasAnyFlags(KeyIndex, ECollectionPropertyFlags::Intrinsic); }
		bool IsStringDirty(int32 KeyIndex) const { return HasAnyFlags(KeyIndex, ECollectionPropertyFlags::StringDirty); }
		bool IsDirty(int32 KeyIndex) const { return HasAnyFlags(KeyIndex, ECollectionPropertyFlags::Dirty); }
		bool IsInterpolable(int32 KeyIndex) const { return HasAnyFlags(KeyIndex, ECollectionPropertyFlags::Interpolable); }

		//~ Values access per key
		template<typename T UE_REQUIRES(TIsWeightedType<T>::Value)>
		T GetLowValue(const FName& Key, const T& Default = T(0), int32* OutKeyIndex = nullptr) const
		{
			return SafeGet(Key, [this](int32 KeyIndex)->T { return GetLowValue<T>(KeyIndex); }, Default, OutKeyIndex);
		}

		template<typename T UE_REQUIRES(TIsWeightedType<T>::Value)>
		T GetHighValue(const FName& Key, const T& Default = T(0), int32* OutKeyIndex = nullptr) const
		{
			return SafeGet(Key, [this](int32 KeyIndex)->T { return GetHighValue<T>(KeyIndex); }, Default, OutKeyIndex);
		}

		template<typename T UE_REQUIRES(TIsWeightedType<T>::Value)>
		TPair<T, T> GetWeightedValue(const FName& Key, const T& Default = T(0), int32* OutKeyIndex = nullptr) const
		{
			return SafeGet(Key, [this](int32 KeyIndex)->TPair<T, T> { return GetWeightedValue<T>(KeyIndex); }, MakeTuple(Default, Default), OutKeyIndex);
		}

		FVector2f GetWeightedFloatValue(const FName& Key, const float& Default = 0.f, int32* OutKeyIndex = nullptr) const
		{
			return SafeGet(Key, [this](int32 KeyIndex)->FVector2f { return GetWeightedFloatValue(KeyIndex); }, FVector2f(Default), OutKeyIndex);
		}

		FVector2f GetWeightedFloatValue(const FName& Key, const FVector2f& Default, int32* OutKeyIndex = nullptr) const
		{
			return SafeGet(Key, [this](int32 KeyIndex)->FVector2f { return GetWeightedFloatValue(KeyIndex); }, Default, OutKeyIndex);
		}

		template<typename T UE_REQUIRES(TIsWeightedType<T>::Value)>
		T GetValue(const FName& Key, const T& Default = T(0), int32* OutKeyIndex = nullptr) const
		{
			return SafeGet(Key, [this](int32 KeyIndex)->T { return GetValue<T>(KeyIndex); }, Default, OutKeyIndex);
		}

		FString GetStringValue(const FName& Key, const FString& Default = "", int32* OutKeyIndex = nullptr) const
		{
			return SafeGet(Key, [this](int32 KeyIndex)->FString { return GetStringValue(KeyIndex); }, Default, OutKeyIndex);
		}

		ECollectionPropertyFlags GetFlags(const FName& Key, uint8 Default = 0, int32* OutKeyIndex = nullptr) const
		{
			return SafeGet(Key, [this](int32 KeyIndex)->ECollectionPropertyFlags { return GetFlags(KeyIndex); }, (ECollectionPropertyFlags)Default, OutKeyIndex);
		}

		bool IsEnabled(const FName& Key, bool bDefault = false, int32* OutKeyIndex = nullptr) const
		{
			return SafeGet(Key, [this](int32 KeyIndex)->bool { return IsEnabled(KeyIndex); }, bDefault, OutKeyIndex);
		}

		bool IsAnimatable(const FName& Key, bool bDefault = false, int32* OutKeyIndex = nullptr) const
		{
			return SafeGet(Key, [this](int32 KeyIndex)->bool { return IsAnimatable(KeyIndex); }, bDefault, OutKeyIndex);
		}

		bool IsLegacy(const FName& Key, bool bDefault = false, int32* OutKeyIndex = nullptr) const
		{
			return SafeGet(Key, [this](int32 KeyIndex)->bool { return IsLegacy(KeyIndex); }, bDefault, OutKeyIndex);
		}

		bool IsIntrinsic(const FName& Key, bool bDefault = false, int32* OutKeyIndex = nullptr) const
		{
			return SafeGet(Key, [this](int32 KeyIndex)->bool { return IsIntrinsic(KeyIndex); }, bDefault, OutKeyIndex);
		}

		bool IsStringDirty(const FName& Key, bool bDefault = false, int32* OutKeyIndex = nullptr) const
		{
			return SafeGet(Key, [this](int32 KeyIndex)->bool { return IsStringDirty(KeyIndex); }, bDefault, OutKeyIndex);
		}

		bool IsDirty(const FName& Key, bool bDefault = false, int32* OutKeyIndex = nullptr) const
		{
			return SafeGet(Key, [this](int32 KeyIndex)->bool { return IsDirty(KeyIndex); }, bDefault, OutKeyIndex);
		}
		
		bool IsInterpolable(const FName& Key, bool bDefault = false, int32* OutKeyIndex = nullptr) const
		{
			return SafeGet(Key, [this](int32 KeyIndex)->bool { return IsInterpolable(KeyIndex); }, bDefault, OutKeyIndex);
		}
		
		//~ Deprecated string key versions: Values access per key
		template<typename T, typename StringType UE_REQUIRES(TIsWeightedType<T>::Value && TIsStringAndNameConvertibleType<StringType>::Value)>
		UE_DEPRECATED(5.7, "Use FName Key version of this method")
		T GetLowValue(const StringType Key, const T& Default = T(0), int32* OutKeyIndex = nullptr) const
		{
			return SafeGet(FName(Key), [this](int32 KeyIndex)->T { return GetLowValue<T>(KeyIndex); }, Default, OutKeyIndex);
		}

		template<typename T, typename StringType UE_REQUIRES(TIsWeightedType<T>::Value && TIsStringAndNameConvertibleType<StringType>::Value)>
		UE_DEPRECATED(5.7, "Use FName Key version of this method")
		T GetHighValue(const StringType Key, const T& Default = T(0), int32* OutKeyIndex = nullptr) const
		{
			return SafeGet(FName(Key), [this](int32 KeyIndex)->T { return GetHighValue<T>(KeyIndex); }, Default, OutKeyIndex);
		}

		template<typename T, typename StringType UE_REQUIRES(TIsWeightedType<T>::Value && TIsStringAndNameConvertibleType<StringType>::Value)>
		UE_DEPRECATED(5.7, "Use FName Key version of this method")
		TPair<T, T> GetWeightedValue(const StringType Key, const T& Default = T(0), int32* OutKeyIndex = nullptr) const
		{
			return SafeGet(FName(Key), [this](int32 KeyIndex)->TPair<T, T> { return GetWeightedValue<T>(KeyIndex); }, MakeTuple(Default, Default), OutKeyIndex);
		}

		template<typename StringType UE_REQUIRES(TIsStringAndNameConvertibleType<StringType>::Value)>
		UE_DEPRECATED(5.7, "Use FName Key version of this method")
		FVector2f GetWeightedFloatValue(const StringType Key, const float& Default = 0.f, int32* OutKeyIndex = nullptr) const
		{
			return SafeGet(FName(Key), [this](int32 KeyIndex)->FVector2f { return GetWeightedFloatValue(KeyIndex); }, FVector2f(Default), OutKeyIndex);
		}

		template<typename StringType UE_REQUIRES(TIsStringAndNameConvertibleType<StringType>::Value)>
		UE_DEPRECATED(5.7, "Use FName Key version of this method")
		FVector2f GetWeightedFloatValue(const StringType Key, const FVector2f& Default, int32* OutKeyIndex = nullptr) const
		{
			return SafeGet(FName(Key), [this](int32 KeyIndex)->FVector2f { return GetWeightedFloatValue(KeyIndex); }, Default, OutKeyIndex);
		}

		template<typename T, typename StringType UE_REQUIRES(TIsWeightedType<T>::Value && TIsStringAndNameConvertibleType<StringType>::Value)>
		UE_DEPRECATED(5.7, "Use FName Key version of this method")
		T GetValue(const StringType Key, const T& Default = T(0), int32* OutKeyIndex = nullptr) const
		{
			return SafeGet(FName(Key), [this](int32 KeyIndex)->T { return GetValue<T>(KeyIndex); }, Default, OutKeyIndex);
		}

		template<typename StringType UE_REQUIRES(TIsStringAndNameConvertibleType<StringType>::Value)>
		UE_DEPRECATED(5.7, "Use FName Key version of this method")
		FString GetStringValue(const StringType Key, const FString& Default = "", int32* OutKeyIndex = nullptr) const
		{
			return SafeGet(FName(Key), [this](int32 KeyIndex)->FString { return GetStringValue(KeyIndex); }, Default, OutKeyIndex);
		}

		template<typename StringType UE_REQUIRES(TIsStringAndNameConvertibleType<StringType>::Value)>
		UE_DEPRECATED(5.7, "Use FName Key version of this method")
		ECollectionPropertyFlags GetFlags(const StringType Key, uint8 Default = 0, int32* OutKeyIndex = nullptr) const
		{
			return SafeGet(FName(Key), [this](int32 KeyIndex)->ECollectionPropertyFlags { return GetFlags(KeyIndex); }, (ECollectionPropertyFlags)Default, OutKeyIndex);
		}

		template<typename StringType UE_REQUIRES(TIsStringAndNameConvertibleType<StringType>::Value)>
		UE_DEPRECATED(5.7, "Use FName Key version of this method")
		bool IsEnabled(const StringType Key, bool bDefault = false, int32* OutKeyIndex = nullptr) const
		{
			return SafeGet(FName(Key), [this](int32 KeyIndex)->bool { return IsEnabled(KeyIndex); }, bDefault, OutKeyIndex);
		}

		template<typename StringType UE_REQUIRES(TIsStringAndNameConvertibleType<StringType>::Value)>
		UE_DEPRECATED(5.7, "Use FName Key version of this method")
		bool IsAnimatable(const StringType Key, bool bDefault = false, int32* OutKeyIndex = nullptr) const
		{
			return SafeGet(FName(Key), [this](int32 KeyIndex)->bool { return IsAnimatable(KeyIndex); }, bDefault, OutKeyIndex);
		}

		template<typename StringType UE_REQUIRES(TIsStringAndNameConvertibleType<StringType>::Value)>
		UE_DEPRECATED(5.7, "Use FName Key version of this method")
		bool IsLegacy(const StringType Key, bool bDefault = false, int32* OutKeyIndex = nullptr) const
		{
			return SafeGet(FName(Key), [this](int32 KeyIndex)->bool { return IsLegacy(KeyIndex); }, bDefault, OutKeyIndex);
		}

		template<typename StringType UE_REQUIRES(TIsStringAndNameConvertibleType<StringType>::Value)>
		UE_DEPRECATED(5.7, "Use FName Key version of this method")
		bool IsIntrinsic(const StringType Key, bool bDefault = false, int32* OutKeyIndex = nullptr) const
		{
			return SafeGet(FName(Key), [this](int32 KeyIndex)->bool { return IsIntrinsic(KeyIndex); }, bDefault, OutKeyIndex);
		}

		template<typename StringType UE_REQUIRES(TIsStringAndNameConvertibleType<StringType>::Value)>
		UE_DEPRECATED(5.7, "Use FName Key version of this method")
		bool IsStringDirty(const StringType Key, bool bDefault = false, int32* OutKeyIndex = nullptr) const
		{
			return SafeGet(FName(Key), [this](int32 KeyIndex)->bool { return IsStringDirty(KeyIndex); }, bDefault, OutKeyIndex);
		}

		template<typename StringType UE_REQUIRES(TIsStringAndNameConvertibleType<StringType>::Value)>
		UE_DEPRECATED(5.7, "Use FName Key version of this method")
		bool IsDirty(const StringType Key, bool bDefault = false, int32* OutKeyIndex = nullptr) const
		{
			return SafeGet(FName(Key), [this](int32 KeyIndex)->bool { return IsDirty(KeyIndex); }, bDefault, OutKeyIndex);
		}

		template<typename StringType UE_REQUIRES(TIsStringAndNameConvertibleType<StringType>::Value)>
		UE_DEPRECATED(5.7, "Use FName Key version of this method")
		bool IsInterpolable(const StringType Key, bool bDefault = false, int32* OutKeyIndex = nullptr) const
		{
			return SafeGet(FName(Key), [this](int32 KeyIndex)->bool { return IsInterpolable(KeyIndex); }, bDefault, OutKeyIndex);
		}

		friend ::uint32 GetTypeHash(const Chaos::Softs::FCollectionPropertyConstFacade& PropertyFacade)
		{
			uint32 Hash = 0;
			if (PropertyFacade.IsValid())
			{
				Hash = GetArrayHash(PropertyFacade.KeyNameArray.GetData(), PropertyFacade.KeyNameArray.Num(), Hash);
				Hash = GetArrayHash(PropertyFacade.LowValueArray.GetData(), PropertyFacade.LowValueArray.Num(), Hash);
				Hash = GetArrayHash(PropertyFacade.HighValueArray.GetData(), PropertyFacade.HighValueArray.Num(), Hash);
				Hash = GetArrayHash(PropertyFacade.StringValueArray.GetData(), PropertyFacade.StringValueArray.Num(), Hash);
				Hash = GetArrayHash(PropertyFacade.FlagsArray.GetData(), PropertyFacade.FlagsArray.Num(), Hash);
			}
			return Hash;
		}

		/** Return whether two properties from different facades have the same values regardless of their names or types. */
		static bool AreValuesEqual(const FCollectionPropertyConstFacade& Facade, int32 KeyIndex, const FCollectionPropertyConstFacade& Other, int32 OtherKeyIndex)
		{
			return
				Facade.GetLowValue<FVector3f>(KeyIndex) == Other.GetLowValue<FVector3f>(OtherKeyIndex) &&
				Facade.GetHighValue<FVector3f>(KeyIndex) == Other.GetHighValue<FVector3f>(OtherKeyIndex) &&
				Facade.GetStringValue(KeyIndex) == Other.GetStringValue(OtherKeyIndex) &&
				EnumOnlyContainsFlags(Facade.GetFlags(KeyIndex) ^ Other.GetFlags(OtherKeyIndex), ECollectionPropertyFlags::Dirty | ECollectionPropertyFlags::StringDirty);
		}

		/** Return whether two properties from different facades and sharing the same key index have the same values regardless of their names or types. */
		static bool AreValuesEqual(const FCollectionPropertyConstFacade& Facade, const FCollectionPropertyConstFacade& Other, int32 KeyIndex)
		{
			return AreValuesEqual(Facade, KeyIndex, Other, KeyIndex);
		}

	protected:
		// No init constructor for FCollectionPropertyFacade
		CHAOS_API FCollectionPropertyConstFacade(const TSharedPtr<const FManagedArrayCollection>& InManagedArrayCollection, ENoInit);

		// Update the array views
		CHAOS_API void UpdateArrays();

		// Rebuild the search map
		CHAOS_API void RebuildKeyIndices();

		template<typename CallableType, typename ReturnType>
		ReturnType SafeGet(const FName& Key, CallableType&& Callable, ReturnType Default, int32* OutKeyIndex) const
		{
			const int32 KeyIndex = GetKeyNameIndex(Key);
			if (OutKeyIndex)
			{
				*OutKeyIndex = KeyIndex;
			}
			return KeyIndex != INDEX_NONE ? Callable(KeyIndex) : Default;
		}

		template<typename CallableType, typename ReturnType, typename StringKeyType UE_REQUIRES(TIsStringAndNameConvertibleType<StringKeyType>::Value)>
		UE_DEPRECATED(5.7, "Use FName Key version of this method")
		ReturnType SafeGet(const StringKeyType Key, CallableType&& Callable, ReturnType Default, int32* OutKeyIndex) const
		{
			return SafeGet(FName(Key), Callable, Default, OutKeyIndex);
		}

		template<typename T, typename ElementType>
		CHAOS_API T GetValue(int32 KeyIndex, const TConstArrayView<ElementType>& ValueArray) const;

		template <typename T>
		CHAOS_API TConstArrayView<T> GetArray(const FName& Name) const;

		bool HasAnyFlags(int32 KeyIndex, ECollectionPropertyFlags Flags) const { return EnumHasAnyFlags(GetValue<ECollectionPropertyFlags, uint8>(KeyIndex, FlagsArray), Flags); }

		// Property Group array views
		TConstArrayView<FName> KeyNameArray;
		TConstArrayView<FVector3f> LowValueArray;
		TConstArrayView<FVector3f> HighValueArray;
		TConstArrayView<FString> StringValueArray;
		TConstArrayView<uint8> FlagsArray;

		// Key to index search map
		TMap<FName, int32> KeyNameIndices;

		// Property collection
		TSharedPtr<const FManagedArrayCollection> ManagedArrayCollection;

		UE_DEPRECATED(5.7, "Use KeyNameName instead")
		static CHAOS_API const FName KeyName;
		UE_DEPRECATED(5.7, "Use KeyNameArray instead")
		TConstArrayView<FString> KeyArray;
		UE_DEPRECATED(5.7, "Use KeyNameIndices instead")
		TMap<FString, int32> KeyIndices;
	};

	/**
	 * Defines common API for reading and writing simulation properties data and metadata.
	 * This is mainly used for the cloth simulation properties to provide
	 * weighted values that works in conjunction with weight maps.
	 * 
	 * Note: Int property values are stored as float and therefore limited to 24 bits (-16777215 to 16777215).
	 */
	class FCollectionPropertyFacade : public FCollectionPropertyConstFacade
	{
	public:
		CHAOS_API explicit FCollectionPropertyFacade(const TSharedPtr<FManagedArrayCollection>& InManagedArrayCollection);
		virtual ~FCollectionPropertyFacade() = default;

		FCollectionPropertyFacade() = delete;
		FCollectionPropertyFacade(const FCollectionPropertyFacade&) = delete;
		FCollectionPropertyFacade& operator=(const FCollectionPropertyFacade&) = delete;

		FCollectionPropertyFacade(FCollectionPropertyFacade&&) = default;
		FCollectionPropertyFacade& operator=(FCollectionPropertyFacade&&) = default;

		//~ Values set per index
		template<typename T UE_REQUIRES(TIsWeightedType<T>::Value)>
		void SetLowValue(int32 KeyIndex, const T& Value) { SetValue(KeyIndex, ModifyLowValueArray(), FVector3f(Value)); }

		template<typename T UE_REQUIRES(TIsWeightedType<T>::Value)>
		void SetHighValue(int32 KeyIndex, const T& Value) { SetValue(KeyIndex, ModifyHighValueArray(), FVector3f(Value)); }

		template<typename T UE_REQUIRES(TIsWeightedType<T>::Value)>
		void SetWeightedValue(int32 KeyIndex, const T& LowValue, const T& HighValue) { SetLowValue(KeyIndex, LowValue); SetHighValue(KeyIndex, HighValue); }

		void SetWeightedFloatValue(int32 KeyIndex, const FVector2f& Value) { SetLowValue<float>(KeyIndex, Value.X); SetHighValue<float>(KeyIndex, Value.Y); }

		template<typename T UE_REQUIRES(TIsWeightedType<T>::Value)>
		void SetValue(int32 KeyIndex, const T& Value) { SetWeightedValue(KeyIndex, Value, Value); }

		void SetStringValue(int32 KeyIndex, const FString& Value) { if (StringValueArray[KeyIndex] != Value) { ModifyStringValueArray()[KeyIndex] = Value; SetStringDirty(KeyIndex); } }

		/** SetFlags cannot be used to remove Dirty, StringDirty, Interpolable or Intrinsic flags. Use ClearDirtyFlags to remove dirty flags. */
		CHAOS_API void SetFlags(int32 KeyIndex, ECollectionPropertyFlags Flags);

		void SetEnabled(int32 KeyIndex, bool bEnabled) { EnableFlags(KeyIndex, ECollectionPropertyFlags::Enabled, bEnabled); }
		void SetAnimatable(int32 KeyIndex, bool bAnimatable) { EnableFlags(KeyIndex, ECollectionPropertyFlags::Animatable, bAnimatable); }
		void SetLegacy(int32 KeyIndex, bool bLegacy) { EnableFlags(KeyIndex, ECollectionPropertyFlags::Legacy, bLegacy); }

		/** Set the intrinsic flag for this property. This flag cannot be removed and implies non Animatable. */
		void SetIntrinsic(int32 KeyIndex) { EnableFlags(KeyIndex, ECollectionPropertyFlags::Intrinsic, true); }
		void SetDirty(int32 KeyIndex) { EnableFlags(KeyIndex, ECollectionPropertyFlags::Dirty, true); }
		void SetStringDirty(int32 KeyIndex) { EnableFlags(KeyIndex, ECollectionPropertyFlags::StringDirty, true); }
		void SetInterpolable(int32 KeyIndex) { EnableFlags(KeyIndex, ECollectionPropertyFlags::Interpolable, true); }

		//~ Values set per key
		template<typename T UE_REQUIRES(TIsWeightedType<T>::Value)>
		int32 SetLowValue(const FName& Key, const T& Value)
		{
			return SafeSet(Key, [this, &Value](int32 KeyIndex) { SetLowValue(KeyIndex, Value); });
		}

		template<typename T UE_REQUIRES(TIsWeightedType<T>::Value)>
		int32 SetHighValue(const FName& Key, const T& Value)
		{
			return SafeSet(Key, [this, &Value](int32 KeyIndex) { SetHighValue(KeyIndex, Value); });
		}

		template<typename T UE_REQUIRES(TIsWeightedType<T>::Value)>
		int32 SetWeightedValue(const FName& Key, const T& LowValue, const T& HighValue)
		{
			return SafeSet(Key, [this, &LowValue, &HighValue](int32 KeyIndex) { SetWeightedValue(KeyIndex, LowValue, HighValue); });
		}

		int32 SetWeightedFloatValue(const FName& Key, const FVector2f& Value)
		{
			return SafeSet(Key, [this, &Value](int32 KeyIndex) { SetWeightedFloatValue(KeyIndex, Value); });
		}

		template<typename T UE_REQUIRES(TIsWeightedType<T>::Value)>
		int32 SetValue(const FName& Key, const T& Value)
		{
			return SafeSet(Key, [this, &Value](int32 KeyIndex) { SetValue(KeyIndex, Value); });
		}

		int32 SetStringValue(const FName& Key, const FString& Value)
		{
			return SafeSet(Key, [this, &Value](int32 KeyIndex) { SetStringValue(KeyIndex, Value); });
		}

		int32 SetFlags(const FName& Key, ECollectionPropertyFlags Flags)
		{
			return SafeSet(Key, [this, Flags](int32 KeyIndex) { SetFlags(KeyIndex, Flags); });
		}

		int32 SetEnabled(const FName& Key, bool bEnabled)
		{
			return SafeSet(Key, [this, bEnabled](int32 KeyIndex) { SetEnabled(KeyIndex, bEnabled); });
		}

		int32 SetAnimatable(const FName& Key, bool bAnimatable)
		{
			return SafeSet(Key, [this, bAnimatable](int32 KeyIndex) { SetAnimatable(KeyIndex, bAnimatable); });
		}

		int32 SetLegacy(const FName& Key, bool bLegacy)
		{
			return SafeSet(Key, [this, bLegacy](int32 KeyIndex) { SetLegacy(KeyIndex, bLegacy); });
		}

		/** Set the intrinsic flag for this property. This flag cannot be removed and implies non Animatable. */
		int32 SetIntrinsic(const FName& Key)
		{
			return SafeSet(Key, [this](int32 KeyIndex) { SetIntrinsic(KeyIndex); });
		}

		int32 SetDirty(const FName& Key)
		{
			return SafeSet(Key, [this](int32 KeyIndex) { SetDirty(KeyIndex); });
		}

		int32 SetStringDirty(const FName& Key)
		{
			return SafeSet(Key, [this](int32 KeyIndex) { SetStringDirty(KeyIndex); });
		}
		
		int32 SetInterpolable(const FName& Key)
		{
			return SafeSet(Key, [this](int32 KeyIndex) { SetInterpolable(KeyIndex); });
		}

		//~ Deprecated string key versions: Values set per key
		template<typename T, typename StringType UE_REQUIRES(TIsWeightedType<T>::Value && TIsStringAndNameConvertibleType<StringType>::Value)>
		UE_DEPRECATED(5.7, "Use FName Key version of this method")
		int32 SetLowValue(const StringType Key, const T& Value)
		{
			return SafeSet(FName(Key), [this, &Value](int32 KeyIndex) { SetLowValue(KeyIndex, Value); });
		}

		template<typename T, typename StringType UE_REQUIRES(TIsWeightedType<T>::Value && TIsStringAndNameConvertibleType<StringType>::Value)>
		UE_DEPRECATED(5.7, "Use FName Key version of this method")
		int32 SetHighValue(const StringType Key, const T& Value)
		{
			return SafeSet(FName(Key), [this, &Value](int32 KeyIndex) { SetHighValue(KeyIndex, Value); });
		}

		template<typename T, typename StringType UE_REQUIRES(TIsWeightedType<T>::Value && TIsStringAndNameConvertibleType<StringType>::Value)>
		UE_DEPRECATED(5.7, "Use FName Key version of this method")
		int32 SetWeightedValue(const StringType Key, const T& LowValue, const T& HighValue)
		{
			return SafeSet(FName(Key), [this, &LowValue, &HighValue](int32 KeyIndex) { SetWeightedValue(KeyIndex, LowValue, HighValue); });
		}

		template<typename StringType UE_REQUIRES(TIsStringAndNameConvertibleType<StringType>::Value)>
		UE_DEPRECATED(5.7, "Use FName Key version of this method")
		int32 SetWeightedFloatValue(const StringType Key, const FVector2f& Value)
		{
			return SafeSet(FName(Key), [this, &Value](int32 KeyIndex) { SetWeightedFloatValue(KeyIndex, Value); });
		}

		template<typename T, typename StringType UE_REQUIRES(TIsWeightedType<T>::Value && TIsStringAndNameConvertibleType<StringType>::Value)>
		UE_DEPRECATED(5.7, "Use FName Key version of this method")
		int32 SetValue(const StringType Key, const T& Value)
		{
			return SafeSet(FName(Key), [this, &Value](int32 KeyIndex) { SetValue(KeyIndex, Value); });
		}

		template<typename StringType UE_REQUIRES(TIsStringAndNameConvertibleType<StringType>::Value)>
		UE_DEPRECATED(5.7, "Use FName Key version of this method")
		int32 SetStringValue(const StringType Key, const FString& Value)
		{
			return SafeSet(FName(Key), [this, &Value](int32 KeyIndex) { SetStringValue(KeyIndex, Value); });
		}

		template<typename StringType UE_REQUIRES(TIsStringAndNameConvertibleType<StringType>::Value)>
		UE_DEPRECATED(5.7, "Use FName Key version of this method")
		int32 SetFlags(const StringType Key, ECollectionPropertyFlags Flags)
		{
			return SafeSet(FName(Key), [this, Flags](int32 KeyIndex) { SetFlags(KeyIndex, Flags); });
		}

		template<typename StringType UE_REQUIRES(TIsStringAndNameConvertibleType<StringType>::Value)>
		UE_DEPRECATED(5.7, "Use FName Key version of this method")
		int32 SetEnabled(const StringType Key, bool bEnabled)
		{
			return SafeSet(FName(Key), [this, bEnabled](int32 KeyIndex) { SetEnabled(KeyIndex, bEnabled); });
		}

		template<typename StringType UE_REQUIRES(TIsStringAndNameConvertibleType<StringType>::Value)>
		UE_DEPRECATED(5.7, "Use FName Key version of this method")
		int32 SetAnimatable(const StringType Key, bool bAnimatable)
		{
			return SafeSet(FName(Key), [this, bAnimatable](int32 KeyIndex) { SetAnimatable(KeyIndex, bAnimatable); });
		}

		template<typename StringType UE_REQUIRES(TIsStringAndNameConvertibleType<StringType>::Value)>
		UE_DEPRECATED(5.7, "Use FName Key version of this method")
		int32 SetLegacy(const StringType Key, bool bLegacy)
		{
			return SafeSet(FName(Key), [this, bLegacy](int32 KeyIndex) { SetLegacy(KeyIndex, bLegacy); });
		}

		/** Set the intrinsic flag for this property. This flag cannot be removed and implies non Animatable. */
		template<typename StringType UE_REQUIRES(TIsStringAndNameConvertibleType<StringType>::Value)>
		UE_DEPRECATED(5.7, "Use FName Key version of this method")
		int32 SetIntrinsic(const StringType Key)
		{
			return SafeSet(FName(Key), [this](int32 KeyIndex) { SetIntrinsic(KeyIndex); });
		}

		template<typename StringType UE_REQUIRES(TIsStringAndNameConvertibleType<StringType>::Value)>
		UE_DEPRECATED(5.7, "Use FName Key version of this method")
		int32 SetDirty(const StringType Key)
		{
			return SafeSet(FName(Key), [this](int32 KeyIndex) { SetDirty(KeyIndex); });
		}

		template<typename StringType UE_REQUIRES(TIsStringAndNameConvertibleType<StringType>::Value)>
		UE_DEPRECATED(5.7, "Use FName Key version of this method")
		int32 SetStringDirty(const StringType Key)
		{
			return SafeSet(FName(Key), [this](int32 KeyIndex) { SetStringDirty(KeyIndex); });
		}

		template<typename StringType UE_REQUIRES(TIsStringAndNameConvertibleType<StringType>::Value)>
		UE_DEPRECATED(5.7, "Use FName Key version of this method")
		int32 SetInterpolable(const StringType Key)
		{
			return SafeSet(FName(Key), [this](int32 KeyIndex) { SetInterpolable(KeyIndex); });
		}

		/** Clear all individual properties' dirty flags and this collection facade dirty flag. */
		CHAOS_API void ClearDirtyFlags();

		/** Make this collection facade dirty (does not change individual property flags). */
		CHAOS_API void MakeDirty();

		CHAOS_API void UpdateProperties(const TSharedPtr<const FManagedArrayCollection>& InManagedArrayCollection);

	protected:
		// No init constructor for FCollectionPropertyMutableFacade
		CHAOS_API FCollectionPropertyFacade(const TSharedPtr<FManagedArrayCollection>& InManagedArrayCollection, ENoInit);

		// Access to a writeable ManagedArrayCollection is protected, use an FPropertyCollectionMutableAdapter if needed to get a non const pointer
		TSharedPtr<FManagedArrayCollection> GetManagedArrayCollection() { return ConstCastSharedPtr<FManagedArrayCollection>(ManagedArrayCollection); }

		// Mutable array accessors via ModifyAttribute (marks collection dirty)
		CHAOS_API TArrayView<FName> ModifyKeyNameArray();
		CHAOS_API TArrayView<FVector3f> ModifyLowValueArray();
		CHAOS_API TArrayView<FVector3f> ModifyHighValueArray();
		CHAOS_API TArrayView<FString> ModifyStringValueArray();
		CHAOS_API TArrayView<ECollectionPropertyFlags> ModifyFlagsArray();

		UE_DEPRECATED(5.8, "Use ModifyKeyNameArray instead.")
		const TArrayView<FName>& GetKeyNameArray() { return reinterpret_cast<TArrayView<FName>&>(KeyNameArray); }
		UE_DEPRECATED(5.8, "Use ModifyLowValueArray instead.")
		const TArrayView<FVector3f>& GetLowValueArray() { return reinterpret_cast<TArrayView<FVector3f>&>(LowValueArray); }
		UE_DEPRECATED(5.8, "Use ModifyHighValueArray instead.")
		const TArrayView<FVector3f>& GetHighValueArray() { return reinterpret_cast<TArrayView<FVector3f>&>(HighValueArray); }
		UE_DEPRECATED(5.8, "Use ModifyStringValueArray instead.")
		const TArrayView<FString>& GetStringValueArray() { return reinterpret_cast<TArrayView<FString>&>(StringValueArray); }
		UE_DEPRECATED(5.8, "Use ModifyFlagsArray instead.")
		const TArrayView<ECollectionPropertyFlags>& GetFlagsArray() { return reinterpret_cast<const TArrayView<ECollectionPropertyFlags>&>(FlagsArray); }

		UE_DEPRECATED(5.7, "Use GetKeyNameArray instead")
		const TArrayView<FString>& GetKeyArray() 
		{ 
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			return reinterpret_cast<TArrayView<FString>&>(KeyArray); 
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}

	private:
		template<typename CallableType, typename StringType UE_REQUIRES(TIsStringAndNameConvertibleType<StringType>::Value)>
		UE_DEPRECATED(5.7, "Use FName Key version of this method")
		int32 SafeSet(const StringType Key, CallableType&& Callable)
		{
			return SafeSet(FName(Key), Callable);
		}

		template<typename CallableType>
		int32 SafeSet(const FName& Key, CallableType&& Callable)
		{
			const int32 KeyIndex = GetKeyNameIndex(Key);
			if (KeyIndex != INDEX_NONE)
			{
				Callable(KeyIndex);
			}
			return KeyIndex;
		}

		template<typename T>
		inline void SetValue(int32 KeyIndex, const TArrayView<T>& ValueArray, const T& Value);

		CHAOS_API void EnableFlags(int32 KeyIndex, ECollectionPropertyFlags Flags, bool bEnable);

	};

	enum class ECollectionPropertyUpdateFlags : uint8
	{
		None = 0,
		AppendNewProperties = 1 << 0, /** Add new properties.*/
		UpdateExistingProperties = 1 << 1, /** Update values of existing properties.*/
		RemoveMissingProperties = 1 << 2, /** Existing properties not in InManagedArrayCollection will be removed */
		DisableMissingProperties = 1 << 3 /** Existing properties not in InManagedArrayCollection will be disabled. (Does nothing if RemoveMissingProperties is set).*/
	};
	ENUM_CLASS_FLAGS(ECollectionPropertyUpdateFlags)

	/**
	 * Defines common API for reading and writing, and adding/removing simulation properties data and metadata.
	 * This is mainly used for the cloth simulation properties to provide
	 * weighted values that works in conjunction with weight maps.
	 */
	class FCollectionPropertyMutableFacade final : public FCollectionPropertyFacade
	{
	public:
		CHAOS_API explicit FCollectionPropertyMutableFacade(const TSharedPtr<FManagedArrayCollection>& InManagedArrayCollection);
		virtual ~FCollectionPropertyMutableFacade() = default;

		FCollectionPropertyMutableFacade() = delete;
		FCollectionPropertyMutableFacade(const FCollectionPropertyMutableFacade&) = delete;
		FCollectionPropertyMutableFacade& operator=(const FCollectionPropertyMutableFacade&) = delete;

		FCollectionPropertyMutableFacade(FCollectionPropertyMutableFacade&&) = default;
		FCollectionPropertyMutableFacade& operator=(FCollectionPropertyMutableFacade&&) = default;

		/** Create this facade's groups and attributes. */
		CHAOS_API void DefineSchema();

		/** Add a single property, and return its index. */
		CHAOS_API int32 AddProperty(const FName& Key, ECollectionPropertyFlags Flags = ECollectionPropertyFlags::Enabled);
		template<typename StringType UE_REQUIRES(TIsStringAndNameConvertibleType<StringType>::Value)>
		UE_DEPRECATED(5.7, "Use FName Key version of this method")
		int32 AddProperty(const StringType Key, ECollectionPropertyFlags Flags = ECollectionPropertyFlags::Enabled)
		{
			return AddProperty(FName(Key), Flags);
		}

		/** Add a single property, and return its index. */
		CHAOS_API int32 AddProperty(const FName& Key, bool bEnabled, bool bAnimatable = false, bool bIntrinsic = false);
		template<typename StringType UE_REQUIRES(TIsStringAndNameConvertibleType<StringType>::Value)>
		UE_DEPRECATED(5.7, "Use FName Key version of this method")
		int32 AddProperty(const StringType Key, bool bEnabled, bool bAnimatable = false, bool bIntrinsic = false)
		{
			return AddProperty(FName(Key), bEnabled, bAnimatable, bIntrinsic);
		}

		/** Add new properties, and return the index of the first added property. */
		CHAOS_API int32 AddProperties(const TArray<FName>& Keys, ECollectionPropertyFlags Flags = ECollectionPropertyFlags::Enabled);
		template<typename StringType UE_REQUIRES(TIsStringAndNameConvertibleType<StringType>::Value)>
		UE_DEPRECATED(5.7, "Use FName Keys version of this method")
		int32 AddProperties(const TArray<StringType>& InKeys, ECollectionPropertyFlags Flags = ECollectionPropertyFlags::Enabled)
		{
			TArray<FName> Keys;
			Keys.Reserve(Keys.Num());
			for (const StringType& Key : InKeys)
			{
				Keys.Emplace(FName(Key));
			}
			return AddProperties(Keys, Flags);
		}

		/** Add new properties, and return the index of the first added property. */
		CHAOS_API int32 AddProperties(const TArray<FName>& Keys, bool bEnabled, bool bAnimatable = false, bool bIntrinsic = false);
		template<typename StringType UE_REQUIRES(TIsStringAndNameConvertibleType<StringType>::Value)>
		UE_DEPRECATED(5.7, "Use FName Keys version of this method")
		int32 AddProperties(const TArray<StringType>& InKeys, bool bEnabled, bool bAnimatable = false, bool bIntrinsic = false)
		{
			TArray<FName> Keys;
			Keys.Reserve(Keys.Num());
			for (const StringType& Key : InKeys)
			{
				Keys.Emplace(FName(Key));
			}
			return AddProperties(Keys, bEnabled, bAnimatable, bIntrinsic);
		}

		/**
		 * Append all properties and values from an existing collection to this property collection.
		 * This won't copy any other groups, only data from PropertyGroup.
		 * Modified properties will be marked dirty.
		 * 
		 * @param bUpdateExistingProperties: Update existing property values. Otherwise, only new properties will be added.
		 */
		CHAOS_API void Append(const TSharedPtr<const FManagedArrayCollection>& InManagedArrayCollection, bool bUpdateExistingProperties);

		/**
		 * Copy all properties and values from an existing collection to this property collection.
		 * Dirty flags will be copied directly.
		 * This won't copy any other groups, only data from PropertyGroup.
		 * Any pre-xisting data will be removed/replaced.
		 */
		CHAOS_API void Copy(const FManagedArrayCollection& InManagedArrayCollection);

		/**
		 * Update all properties and values from an existing collection to this property collection.
		 * This won't copy any other groups, only data from PropertyGroup.
		 * Modified properties will be marked dirty.
		 */
		CHAOS_API void Update(const TSharedPtr<const FManagedArrayCollection>& InManagedArrayCollection, ECollectionPropertyUpdateFlags UpdateFlags);

		//~ Add values
		template<typename T UE_REQUIRES(TIsWeightedType<T>::Value)>
		inline int32 AddWeightedValue(const FName& Key, const T& LowValue, const T& HighValue, ECollectionPropertyFlags Flags = ECollectionPropertyFlags::Enabled);
		template<typename T UE_REQUIRES(TIsWeightedType<T>::Value)>
		inline int32 AddWeightedValue(const FName& Key, const T& LowValue, const T& HighValue, bool bEnabled, bool bAnimatable = false, bool bIntrinsic = false);

		CHAOS_API int32 AddWeightedFloatValue(const FName& Key, const FVector2f& Value, ECollectionPropertyFlags Flags = ECollectionPropertyFlags::Enabled);
		CHAOS_API int32 AddWeightedFloatValue(const FName& Key, const FVector2f& Value, bool bEnabled, bool bAnimatable, bool bIntrinsic = false);

		template<typename T UE_REQUIRES(TIsWeightedType<T>::Value)>
		int32 AddValue(const FName& Key, const T& Value, ECollectionPropertyFlags Flags = ECollectionPropertyFlags::Enabled) { return AddWeightedValue(Key, Value, Value, Flags); }
		template<typename T UE_REQUIRES(TIsWeightedType<T>::Value)>
		int32 AddValue(const FName& Key, const T& Value, bool bEnabled, bool bAnimatable = false) { return AddWeightedValue(Key, Value, Value, bEnabled, bAnimatable); }

		CHAOS_API int32 AddStringValue(const FName& Key, const FString& Value, ECollectionPropertyFlags Flags = ECollectionPropertyFlags::Enabled);
		CHAOS_API int32 AddStringValue(const FName& Key, const FString& Value, bool bEnabled, bool bAnimatable = false, bool bIntrinsic = false);

		//~ Deprecated string key versions: Add values
		template<typename T, typename StringType UE_REQUIRES(TIsWeightedType<T>::Value&& TIsStringAndNameConvertibleType<StringType>::Value)>
		UE_DEPRECATED(5.7, "Use FName Key version of this method")
		int32 AddWeightedValue(const StringType Key, const T& LowValue, const T& HighValue, ECollectionPropertyFlags Flags = ECollectionPropertyFlags::Enabled)
		{
			return AddWeightedValue(FName(Key), LowValue, HighValue, Flags);
		}

		template<typename T, typename StringType UE_REQUIRES(TIsWeightedType<T>::Value&& TIsStringAndNameConvertibleType<StringType>::Value)>
		UE_DEPRECATED(5.7, "Use FName Key version of this method")
		int32 AddWeightedValue(const StringType Key, const T& LowValue, const T& HighValue, bool bEnabled, bool bAnimatable = false, bool bIntrinsic = false)
		{
			return AddWeightedValue(FName(Key), LowValue, HighValue, bEnabled, bAnimatable, bIntrinsic);
		}

		template<typename StringType UE_REQUIRES(TIsStringAndNameConvertibleType<StringType>::Value)>
		UE_DEPRECATED(5.7, "Use FName Key version of this method")
		int32 AddWeightedFloatValue(const StringType Key, const FVector2f& Value, ECollectionPropertyFlags Flags = ECollectionPropertyFlags::Enabled)
		{
			return AddWeightedFloatValue(FName(Key), Value, Flags);
		}

		template<typename StringType UE_REQUIRES(TIsStringAndNameConvertibleType<StringType>::Value)>
		UE_DEPRECATED(5.7, "Use FName Key version of this method")
		int32 AddWeightedFloatValue(const StringType Key, const FVector2f& Value, bool bEnabled, bool bAnimatable, bool bIntrinsic = false)
		{
			return AddWeightedFloatValue(FName(Key), Value, bEnabled, bAnimatable, bIntrinsic);
		}

		template<typename T, typename StringType UE_REQUIRES(TIsWeightedType<T>::Value&& TIsStringAndNameConvertibleType<StringType>::Value)>
		UE_DEPRECATED(5.7, "Use FName Key version of this method")
		int32 AddValue(const StringType Key, const T& Value, ECollectionPropertyFlags Flags = ECollectionPropertyFlags::Enabled)
		{
			return AddWeightedValue(FName(Key), Value, Value, Flags);
		}

		template<typename T, typename StringType UE_REQUIRES(TIsWeightedType<T>::Value&& TIsStringAndNameConvertibleType<StringType>::Value)>
		UE_DEPRECATED(5.7, "Use FName Key version of this method")
		int32 AddValue(const StringType Key, const T& Value, bool bEnabled, bool bAnimatable = false)
		{ 
			return AddWeightedValue(FName(Key), Value, Value, bEnabled, bAnimatable); 
		}

		template<typename StringType UE_REQUIRES(TIsStringAndNameConvertibleType<StringType>::Value)>
		UE_DEPRECATED(5.7, "Use FName Key version of this method")
		int32 AddStringValue(const StringType Key, const FString& Value, ECollectionPropertyFlags Flags = ECollectionPropertyFlags::Enabled)
		{
			return AddStringValue(FName(Key), Value, Flags);
		}

		template<typename StringType UE_REQUIRES(TIsStringAndNameConvertibleType<StringType>::Value)>
		UE_DEPRECATED(5.7, "Use FName Key version of this method")
		int32 AddStringValue(const StringType Key, const FString& Value, bool bEnabled, bool bAnimatable = false, bool bIntrinsic = false)
		{
			return AddStringValue(FName(Key), Value, bEnabled, bAnimatable, bIntrinsic);
		}

		CHAOS_API void PostSerialize(const FArchive& Ar);
	};

	template<typename T>
	inline void FCollectionPropertyFacade::SetValue(int32 KeyIndex, const TArrayView<T>& ValueArray, const T& Value)
	{
		if (ValueArray[KeyIndex] != Value)
		{
			ValueArray[KeyIndex] = Value;
			SetDirty(KeyIndex);
		}
	}

	template<typename T UE_REQUIRES_DEFINITION(TIsWeightedType<T>::Value)>
	inline int32 FCollectionPropertyMutableFacade::AddWeightedValue(const FName& Key, const T& LowValue, const T& HighValue, ECollectionPropertyFlags Flags)
	{
		const int32 KeyIndex = AddProperty(Key, Flags);
		SetWeightedValue(KeyIndex, LowValue, HighValue);
		return KeyIndex;
	}

	template<typename T UE_REQUIRES_DEFINITION(TIsWeightedType<T>::Value)>
	inline int32 FCollectionPropertyMutableFacade::AddWeightedValue(const FName& Key, const T& LowValue, const T& HighValue, bool bEnabled, bool bAnimatable, bool bIntrinsic)
	{
		const int32 KeyIndex = AddProperty(Key, bEnabled, bAnimatable, bIntrinsic);
		SetWeightedValue(KeyIndex, LowValue, HighValue);
		return KeyIndex;
	}
}  // End namespace Chaos

// Use this macro to add shorthands for property getters without a key index
#define UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(PropertyName, Type) \
	inline static const FName PropertyName##Name = TEXT(#PropertyName); \
	static bool Is##PropertyName##Enabled(const ::Chaos::Softs::FCollectionPropertyConstFacade& InPropertyCollection, bool bDefault) \
	{ \
		return InPropertyCollection.IsEnabled(PropertyName##Name, bDefault); \
	} \
	static bool Is##PropertyName##Animatable(const ::Chaos::Softs::FCollectionPropertyConstFacade& InPropertyCollection, bool bDefault) \
	{ \
		return InPropertyCollection.IsAnimatable(PropertyName##Name, bDefault); \
	} \
	static Type GetLow##PropertyName(const ::Chaos::Softs::FCollectionPropertyConstFacade& InPropertyCollection, const Type& Default) \
	{ \
		return InPropertyCollection.GetLowValue<Type>(PropertyName##Name, Default); \
	} \
	static Type GetHigh##PropertyName(const ::Chaos::Softs::FCollectionPropertyConstFacade& InPropertyCollection, const Type& Default) \
	{ \
		return InPropertyCollection.GetHighValue<Type>(PropertyName##Name, Default); \
	} \
	static TPair<Type, Type> GetWeighted##PropertyName(const ::Chaos::Softs::FCollectionPropertyConstFacade& InPropertyCollection, const Type& Default) \
	{ \
		return InPropertyCollection.GetWeightedValue<Type>(PropertyName##Name, Default); \
	} \
	static FVector2f GetWeightedFloat##PropertyName(const ::Chaos::Softs::FCollectionPropertyConstFacade& InPropertyCollection, const float& Default) \
	{ \
		return InPropertyCollection.GetWeightedFloatValue(PropertyName##Name, Default); \
	} \
	static FVector2f GetWeightedFloat##PropertyName(const ::Chaos::Softs::FCollectionPropertyConstFacade& InPropertyCollection, const FVector2f& Default) \
	{ \
		return InPropertyCollection.GetWeightedFloatValue(PropertyName##Name, Default); \
	} \
	static Type Get##PropertyName(const ::Chaos::Softs::FCollectionPropertyConstFacade& InPropertyCollection, const Type& Default) \
	{ \
		return InPropertyCollection.GetValue<Type>(PropertyName##Name, Default); \
	} \
	static FString Get##PropertyName##String(const ::Chaos::Softs::FCollectionPropertyConstFacade& InPropertyCollection, const FString& Default) \
	{ \
		return InPropertyCollection.GetStringValue(PropertyName##Name, Default); \
	} \

// Use this macro to add shorthands for property getters and direct access through the declared key index. 
#define UE_CHAOS_DECLARE_INDEXED_PROPERTYCOLLECTION_NAME(PropertyName, Type) \
	Type GetLow##PropertyName(const ::Chaos::Softs::FCollectionPropertyConstFacade& InPropertyCollection) const \
	{ \
		checkSlow(PropertyName##Index == InPropertyCollection.GetKeyNameIndex(PropertyName##Name)); \
		checkf(PropertyName##Index != INDEX_NONE, TEXT("The default value getter that sets the property index must be called once prior to calling this function.")); \
		return InPropertyCollection.GetLowValue<Type>(PropertyName##Index); \
	} \
	Type GetHigh##PropertyName(const ::Chaos::Softs::FCollectionPropertyConstFacade& InPropertyCollection) const \
	{ \
		checkSlow(PropertyName##Index == InPropertyCollection.GetKeyNameIndex(PropertyName##Name)); \
		checkf(PropertyName##Index != INDEX_NONE, TEXT("The default value getter that sets the property index must be called once prior to calling this function.")); \
		return InPropertyCollection.GetHighValue<Type>(PropertyName##Index); \
	} \
	TPair<Type, Type> GetWeighted##PropertyName(const ::Chaos::Softs::FCollectionPropertyConstFacade& InPropertyCollection) const \
	{ \
		checkSlow(PropertyName##Index == InPropertyCollection.GetKeyNameIndex(PropertyName##Name)); \
		checkf(PropertyName##Index != INDEX_NONE, TEXT("The default value getter that sets the property index must be called once prior to calling this function.")); \
		return InPropertyCollection.GetWeightedValue<Type>(PropertyName##Index); \
	} \
	FVector2f GetWeightedFloat##PropertyName(const ::Chaos::Softs::FCollectionPropertyConstFacade& InPropertyCollection) const \
	{ \
		checkSlow(PropertyName##Index == InPropertyCollection.GetKeyNameIndex(PropertyName##Name)); \
		checkf(PropertyName##Index != INDEX_NONE, TEXT("The default value getter that sets the property index must be called once prior to calling this function.")); \
		return InPropertyCollection.GetWeightedFloatValue(PropertyName##Index); \
	} \
	Type Get##PropertyName(const ::Chaos::Softs::FCollectionPropertyConstFacade& InPropertyCollection) const \
	{ \
		checkSlow(PropertyName##Index == InPropertyCollection.GetKeyNameIndex(PropertyName##Name)); \
		checkf(PropertyName##Index != INDEX_NONE, TEXT("The default value getter that sets the property index must be called once prior to calling this function.")); \
		return InPropertyCollection.GetValue<Type>(PropertyName##Index); \
	} \
	const FString& Get##PropertyName##String(const ::Chaos::Softs::FCollectionPropertyConstFacade& InPropertyCollection) \
	{ \
		checkSlow(PropertyName##Index == InPropertyCollection.GetKeyNameIndex(PropertyName##Name)); \
		checkf(PropertyName##Index != INDEX_NONE, TEXT("The default value getter that sets the property index must be called once prior to calling this function.")); \
		return InPropertyCollection.GetStringValue(PropertyName##Index); \
	} \
	bool Is##PropertyName##Enabled(const ::Chaos::Softs::FCollectionPropertyConstFacade& InPropertyCollection) const \
	{ \
		checkSlow(PropertyName##Index == InPropertyCollection.GetKeyNameIndex(PropertyName##Name)); \
		checkf(PropertyName##Index != INDEX_NONE, TEXT("The default value getter that sets the property index must be called once prior to calling this function.")); \
		return InPropertyCollection.IsEnabled(PropertyName##Index); \
	} \
	bool Is##PropertyName##Animatable(const ::Chaos::Softs::FCollectionPropertyConstFacade& InPropertyCollection) const\
	{ \
		checkSlow(PropertyName##Index == InPropertyCollection.GetKeyNameIndex(PropertyName##Name)); \
		checkf(PropertyName##Index != INDEX_NONE, TEXT("The default value getter that sets the property index must be called once prior to calling this function.")); \
		return InPropertyCollection.IsAnimatable(PropertyName##Index); \
	} \
	bool Is##PropertyName##Dirty(const ::Chaos::Softs::FCollectionPropertyConstFacade& InPropertyCollection) const \
	{ \
		checkSlow(PropertyName##Index == InPropertyCollection.GetKeyNameIndex(PropertyName##Name)); \
		checkf(PropertyName##Index != INDEX_NONE, TEXT("The default value getter that sets the property index must be called once prior to calling this function.")); \
		return InPropertyCollection.IsDirty(PropertyName##Index); \
	} \
	bool Is##PropertyName##StringDirty(const ::Chaos::Softs::FCollectionPropertyConstFacade& InPropertyCollection) const \
	{ \
		checkSlow(PropertyName##Index == InPropertyCollection.GetKeyNameIndex(PropertyName##Name)); \
		checkf(PropertyName##Index != INDEX_NONE, TEXT("The default value getter that sets the property index must be called once prior to calling this function.")); \
		return InPropertyCollection.IsStringDirty(PropertyName##Index); \
	} \
	bool Is##PropertyName##Mutable(const ::Chaos::Softs::FCollectionPropertyConstFacade& InPropertyCollection) const \
	{ \
		return PropertyName##Index != INDEX_NONE && \
			InPropertyCollection.IsAnimatable(PropertyName##Index) && (InPropertyCollection.IsDirty(PropertyName##Index)); \
	} \
	struct F##PropertyName##Index \
	{ \
		int32 Index = INDEX_NONE; \
		explicit F##PropertyName##Index(EForceInit) : Index(INDEX_NONE) {} \
		explicit F##PropertyName##Index(const ::Chaos::Softs::FCollectionPropertyConstFacade& InPropertyCollection) : Index(InPropertyCollection.GetKeyNameIndex(PropertyName##Name)) {} \
		operator int32() const { return Index; } \
	} PropertyName##Index;


// Use this macro to add both indexless and indexed property getters.
#define UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(PropertyName, Type) \
	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(PropertyName, Type) \
	UE_CHAOS_DECLARE_INDEXED_PROPERTYCOLLECTION_NAME(PropertyName, Type) \

