// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/BoundingVolumeUtilities.h"
#include "Chaos/Collision/ParticlePairMidPhase.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/PBDCollisionConstraints.h"
#include "Chaos/PBDRigidsSOAs.h"
#include "ChaosStats.h"

namespace Chaos
{
	/**
	 * Run through a list of particle pairs and pass them onto the collision detector if their AABBs overlap.
	 * In addition, collide all particles in ParticlesA with all particles in ParticlesB.
	 *
	 * No spatial acceleration, and the order is assumed to be already optimized for cache efficiency.
	 */
	class FBasicBroadPhase
	{
	public:
		using FParticleHandle = TGeometryParticleHandle<FReal, 3>;
		using FAABB = FAABB3;

		FBasicBroadPhase(const TArray<FParticlePair>* InParticlePairs, const TArray<FParticleHandle*>* InParticlesA, const TArray<FParticleHandle*>* InParticlesB)
			: ParticlePairs(InParticlePairs)
			, ParticlesA(InParticlesA)
			, ParticlesB(InParticlesB)
		{
		}

		/**
		 *
		 */
		void ProduceOverlaps(
			FReal Dt,
			Private::FCollisionConstraintAllocator* Allocator,
			const FCollisionDetectorSettings& Settings)
		{
			SCOPE_CYCLE_COUNTER(STAT_Collisions_ParticlePairBroadPhase);

			Allocator->SetMaxContexts(1);

			FCollisionContext Context;
			Context.SetSettings(Settings);
			Context.Allocator = Allocator->GetContextAllocator(0);

			if (ParticlePairs != nullptr)
			{
				for (const FParticlePair& ParticlePair : *ParticlePairs)
				{
					// Pair array is const and the particles are not, but we need to const_cast
					FParticleHandle* ParticleA = const_cast<FParticleHandle*>(ParticlePair[0]);
					FParticleHandle* ParticleB = const_cast<FParticleHandle*>(ParticlePair[1]);

					if ((ParticleA != nullptr) && (ParticleB != nullptr))
					{
						ProduceOverlaps(Dt, Context, ParticleA, ParticleB, ParticleA);
					}
				}
			}

			if ((ParticlesA != nullptr) && (ParticlesB != nullptr))
			{
				for (FParticleHandle* ParticleA : *ParticlesA)
				{
					if (ParticleA != nullptr)
					{
						for (FParticleHandle* ParticleB : *ParticlesB)
						{
							if (ParticleB != nullptr)
							{
								ProduceOverlaps(Dt, Context, ParticleA, ParticleB, ParticleA);
							}
						}
					}
				}
			}
		}

	private:
		inline void ProduceOverlaps(
			FReal Dt,
			const FCollisionContext& Context,
			FParticleHandle* ParticleA,
			FParticleHandle* ParticleB,
			FParticleHandle* SearchParticle)
		{
			if (!ParticleA->HasCollision() || !ParticleB->HasCollision())
			{
				return;
			}
			const FAABB3& Box0 = ParticleA->WorldSpaceInflatedBounds();
			const FAABB3& Box1 = ParticleB->WorldSpaceInflatedBounds();
			if (Box0.Intersects(Box1))
			{
				FParticlePairMidPhase* MidPhase = Context.GetAllocator()->GetMidPhase(ParticleA, ParticleB, SearchParticle, Context);
				if (MidPhase != nullptr)
				{
					MidPhase->GenerateCollisions(Context.GetSettings().BoundsExpansion, Dt, Context);
				}
			}
		}

		const TArray<FParticlePair>* ParticlePairs;
		const TArray<FParticleHandle*>* ParticlesA;
		const TArray<FParticleHandle*>* ParticlesB;
	};
}
