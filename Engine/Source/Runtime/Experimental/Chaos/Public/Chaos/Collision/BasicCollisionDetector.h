// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Collision/CollisionDetector.h"
#include "Chaos/Collision/BasicBroadPhase.h"
#include "Chaos/PBDCollisionConstraints.h"

#include "Chaos/ChaosPerfTest.h"
#include "ChaosStats.h"
#include "ProfilingDebugging/CsvProfiler.h"

namespace Chaos
{
	class FBasicCollisionDetector : public FCollisionDetector
	{
	public:
		FBasicCollisionDetector(FBasicBroadPhase& InBroadPhase, FPBDCollisionConstraints& InCollisionContainer)
			: FCollisionDetector(InCollisionContainer)
			, BroadPhase(InBroadPhase)
		{
		}

		FBasicBroadPhase& GetBroadPhase() { return BroadPhase; }

		virtual void DetectCollisions(const FReal Dt, FEvolutionResimCache* Unused) override
		{
			SCOPE_CYCLE_COUNTER(STAT_Collisions_Detect);
			CHAOS_SCOPED_TIMER(DetectCollisions);
			CSV_SCOPED_TIMING_STAT(Chaos, DetectCollisions);

			if (!GetCollisionContainer().GetCollisionsEnabled())
			{
				return;
			}

			CollisionContainer.BeginDetectCollisions();

			// Collision detection pipeline: BroadPhase -> MidPhase ->NarrowPhase -> Container
			BroadPhase.ProduceOverlaps(Dt, &GetCollisionContainer().GetConstraintAllocator(), GetCollisionContainer().GetDetectorSettings());

			CollisionContainer.EndDetectCollisions();
		}

	private:
		FBasicBroadPhase& BroadPhase;
	};

}
