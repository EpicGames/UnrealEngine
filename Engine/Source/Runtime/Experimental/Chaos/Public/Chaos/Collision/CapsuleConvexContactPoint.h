// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/CollisionResolutionTypes.h"
#include "Chaos/ImplicitFwd.h"
#include "Chaos/ImplicitObjectScaled.h"	// Cannot fwd declare scaled implicit

namespace Chaos
{
	extern CHAOS_API FContactPoint CapsuleConvexContactPoint(const FImplicitCapsule3& Capsule, const FRigidTransform3& CapsuleTransform, const FImplicitObject& Convex, const FRigidTransform3& ConvexTransform);
}
