// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Chaos/Core.h"
#include "Chaos/CollisionResolutionTypes.h"
#include "Chaos/Framework/UncheckedArray.h"
#include "Chaos/ImplicitFwd.h"

namespace Chaos
{
	extern CHAOS_API void ConstructCapsuleTriangleOneShotManifold2(const FImplicitCapsule3& Capsule, const FTriangle& Triangle, const FReal CullDistance, FContactPointManifold& OutContactPoints);
}