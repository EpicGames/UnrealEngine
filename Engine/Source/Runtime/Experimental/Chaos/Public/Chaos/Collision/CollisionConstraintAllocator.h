// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "Chaos/Core.h"
#include "Chaos/CollisionResolutionTypes.h"
#include "Chaos/Collision/CollisionContext.h"
#include "Chaos/Collision/CollisionKeys.h"
#include "Chaos/Collision/ParticlePairMidPhase.h"
#include "Chaos/ObjectPool.h"
#include "Chaos/ParticleHandle.h"
#include "ChaosStats.h"


namespace Chaos
{
	class FPBDCollisionConstraints;
	class FSpatialAccelerationBroadPhase;

	namespace Private
	{
		/**
		 * Container the storage for the FCollisionConstraintAllocator, as well as the API to create new midphases and collision constraints.
		 * We have one of these objects per thread on which collisions detection is performed to get lock-free allocations and lists.
		 */
		class FCollisionContextAllocator
		{
		public:
			FCollisionContextAllocator(FPBDCollisionConstraints* InCollisionContainer, const int32 InNumCollisionsPerBlock, const int32 InCurrentEpoch)
				: CollisionContainer(InCollisionContainer)
				, CurrentEpoch(InCurrentEpoch)
#if CHAOS_COLLISION_OBJECTPOOL_ENABLED
				, ConstraintPool(InNumCollisionsPerBlock, 0)
#endif
#if CHAOS_MIDPHASE_OBJECTPOOL_ENABLED
				, MidPhasePool(InNumCollisionsPerBlock, 0)
#endif
			{
			}

			/**
			 * The current epoch used to determine if a collision is up to date
			 */
			int32 GetCurrentEpoch() const
			{
				return CurrentEpoch;
			}

			/**
			 * Create a constraint (called by the MidPhase)
			 */
			CHAOS_API FPBDCollisionConstraintPtr CreateConstraint(
				FGeometryParticleHandle* Particle0,
				const FImplicitObject* Implicit0,
				const FPerShapeData* Shape0,
				const FBVHParticles* Simplicial0,
				const FRigidTransform3& ShapeRelativeTransform0,
				FGeometryParticleHandle* Particle1,
				const FImplicitObject* Implicit1,
				const FPerShapeData* Shape1,
				const FBVHParticles* Simplicial1,
				const FRigidTransform3& ShapeRelativeTransform1,
				const FReal CullDistance,
				const bool bUseManifold,
				const EContactShapesType ShapePairType);

			/**
			 * Create an uninitialized collision constraint (public only for use by Resim which overwrites it with a saved constraint)
			 */
			FPBDCollisionConstraintPtr CreateConstraint()
			{
#if CHAOS_COLLISION_OBJECTPOOL_ENABLED 
				FPBDCollisionConstraint* Constraint = ConstraintPool.Alloc();
				return FPBDCollisionConstraintPtr(Constraint, FPBDCollisionConstraintDeleter(ConstraintPool));
#else
				return MakeUnique<FPBDCollisionConstraint>();
#endif
			}

			/**
			 * Set the constraint as Active - it will be added to the graph and solved this tick.
			 * NOTE: The definitive active constraints list is compiled after the parallel phase 
			 * is complete (FCollisionConstraintAllocator::ProcessNewConstraints).
			 */
			bool ActivateConstraint(FPBDCollisionConstraint* Constraint)
			{
				FPBDCollisionConstraintContainerCookie& Cookie = Constraint->GetContainerCookie();
				// Use the LastUsedEpoch to catch duplicate add attempts. This is still possible
				// via rewind/resim and InjectCollision. Ideally we wouldn't need do update
				// the cookie data until FCollisionConstraintAllocator::ProcessNewCollisions...
				if (Cookie.LastUsedEpoch != CurrentEpoch)
				{
					Cookie.LastUsedEpoch = CurrentEpoch;
					Cookie.ConstraintIndex = INDEX_NONE;
					Cookie.CCDConstraintIndex = INDEX_NONE;

					NewActiveConstraints.Push(Constraint);

					Constraint->Activate();

					return true;
				}
				return false;
			}

			/**
			 * Return a midphase for a particle pair.
			 * This wil create a new midphase if the particle pairs were not recently overlapping, or return an
			 * existing one if they were.
			 * @note Nothing outside of thie allocator should hold a pointer to the midphase, or any constraints
			 * it creates for more than the duration of the tick. Except the IslandManager :|
			*/
			FParticlePairMidPhase* GetMidPhase(FGeometryParticleHandle* Particle0, FGeometryParticleHandle* Particle1, FGeometryParticleHandle* SearchParticlePerformanceHint, const FCollisionContext& Context)
			{
				FParticlePairMidPhase* MidPhase = FindMidPhaseImpl(Particle0, Particle1, SearchParticlePerformanceHint);
				if (MidPhase == nullptr)
				{
					MidPhase = CreateMidPhase(Particle0, Particle1, Context);
				}

				return MidPhase;
			}

			/**
			 * Return a midphase for a particle pair if it already exists, otherwise return null
			*/
			FParticlePairMidPhase* FindMidPhase(FGeometryParticleHandle* Particle0, FGeometryParticleHandle* Particle1, FGeometryParticleHandle* SearchParticlePerformanceHint)
			{
				return FindMidPhaseImpl(Particle0, Particle1, SearchParticlePerformanceHint);
			}

		private:
			friend class Private::FCollisionConstraintAllocator;
			friend class FSortConstraintTask;
			friend class FMergeConstraintsTask;
			friend class Chaos::FSpatialAccelerationBroadPhase;
			

			void Reset()
			{
#if CHAOS_COLLISION_OBJECTPOOL_ENABLED
				ConstraintPool.Reset();
#endif
#if CHAOS_MIDPHASE_OBJECTPOOL_ENABLED
				MidPhasePool.Reset();
#endif
			}

			void BeginDetectCollisions(const int32 InEpoch)
			{
				check(NewActiveConstraints.IsEmpty());
				check(NewMidPhases.IsEmpty());

				CurrentEpoch = InEpoch;
			}

			// Find the midphase for the particle pair if it exists. Every particle holds a list of its midphases. We search "SearchParticle" which should be
			// ideally be the one with fewer midphases on it.
			FParticlePairMidPhase* FindMidPhaseImpl(FGeometryParticleHandle* Particle0, FGeometryParticleHandle* Particle1, FGeometryParticleHandle* SearchParticle)
			{
				check((SearchParticle == Particle0) || (SearchParticle == Particle1));

				const Private::FCollisionParticlePairKey Key = Private::FCollisionParticlePairKey(Particle0, Particle1);
				return SearchParticle->ParticleCollisions().FindMidPhase(Key.GetKey());
			}

			// Create and initialize a midphase for the particle pair. Adds it to the particles' lists of midphases.
			FParticlePairMidPhase* CreateMidPhase(FGeometryParticleHandle* Particle0, FGeometryParticleHandle* Particle1, const FCollisionContext& Context)
			{
				const Private::FCollisionParticlePairKey Key = Private::FCollisionParticlePairKey(Particle0, Particle1);

				// We temporarily hold new midphases as raw pointers and wrap them in a unique ptr later in ProcessNewMidPhases
#if CHAOS_MIDPHASE_OBJECTPOOL_ENABLED 
				FParticlePairMidPhase* MidPhase = MidPhasePool.Alloc();
#else
				FParticlePairMidPhase* MidPhase = FParticlePairMidPhase::Make(Particle0, Particle1);
#endif

				NewMidPhases.Add(MidPhase);

				MidPhase->Init(Particle0, Particle1, Key, Context);
				return MidPhase;
			}

			FPBDCollisionConstraints* CollisionContainer;
			int32 CurrentEpoch;

			TArray<FPBDCollisionConstraint*> NewActiveConstraints;
			TArray<FParticlePairMidPhase*> NewMidPhases;

#if CHAOS_COLLISION_OBJECTPOOL_ENABLED
			FPBDCollisionConstraintPool ConstraintPool;
#endif
#if CHAOS_MIDPHASE_OBJECTPOOL_ENABLED
			FParticlePairMidPhasePool MidPhasePool;
			FParticlePairMidPhasePool MidPhasePool;
#endif
		};


		/**
		 * @brief An allocator and container of collision constraints that supports reuse of constraints from the previous tick
		 * 
		 * All constraint pointers are persistent in memory until Destroy() is called, or until they are pruned.

		 * This allocator maintains the set of all overlapping particle pairs, with each overlapping particle pair managed
		 * by a FParticlePairMidPhase object. the MidPhase object is what actually calls the Narrow Phase and maintains
		 * the set of collision constraints for all the shape pairs on the particles.
		 * 
		 * Constraints are allocated during the collision detection phase and retained between ticks. An attempt to create
		 * a constraint for the same shape pair as seen on the previous tick will return the existing collision constraint with
		 * all of its data intact.
		 *
		 * The allocator also keeps a list of Standard and Swept collision constraints that are active for the current tick.
		 * This list gets reset and rebuilt every frame during collision detection. It may get added to by the IslandManager
		 * if some islands are woken following collision detection.
		 * 
		 * The allocators Epoch counter is used to determine whether a constraint (or midphase object) generated any
		 * contacts for the current frame. When a midphase creates or updates a constraint, it copies the current Epoch
		 * counter.
		 * 
		 * The Midphase list is pruned at the end of each tick so if particles are destroyed or a particle pair is no longer 
		 * overlapping, the collisions will be destroyed.
		 * 
		 * When particles are destroyed, we do not immediately destroy the MidPhases (or Collisions) that are associoated with
		 * the particle. Instead, we clear the particle pointer from them, but leave their destruction to the pruning process.
		 * This avoids the need to parse collision and midphase lists whenever a particle is disabled.
		 * 
		 * NOTE: To reduce RBAN memory use, we do not create any collision blocks until the first call to CreateCollisionConstraint
		 * (see ConstraintPool initialization)
		*/
		class FCollisionConstraintAllocator
		{
		public:
			UE_NONCOPYABLE(FCollisionConstraintAllocator);

			FCollisionConstraintAllocator(const int32 InNumCollisionsPerBlock = 1000)
				: CollisionContainer(nullptr)
				, ContextAllocators()
				, NumCollisionsPerBlock(InNumCollisionsPerBlock)
				, ParticlePairMidPhases()
				, ActiveConstraints()
				, ActiveCCDConstraints()
				, CurrentEpoch(0)
				, bIsDeteministic(false)
				, bInCollisionDetectionPhase(false)
			{
			}

			~FCollisionConstraintAllocator()
			{
				// Explicit clear so we don't have to worry about destruction order of pool and midphases
				Reset();
			}

			void SetCollisionContainer(FPBDCollisionConstraints* InCollisionContainer)
			{
				CollisionContainer = InCollisionContainer;
			}

			void SetMaxContexts(const int32 MaxContexts)
			{
				// Forgot to call SetCollisionContainer after construction
				// @todo(chaos): this check makes the allocator difficult to unit test - fix this
				//check(CollisionContainer != nullptr);

				const int32 NumNewContexts = MaxContexts - ContextAllocators.Num();
				if (NumNewContexts > 0)
				{
					ContextAllocators.Reserve(MaxContexts);
					for (int32 NewIndex = 0; NewIndex < NumNewContexts; ++NewIndex)
					{
						TUniquePtr<FCollisionContextAllocator> ContextAllocator = MakeUnique<FCollisionContextAllocator>(CollisionContainer, NumCollisionsPerBlock, CurrentEpoch);
						ContextAllocators.Emplace(MoveTemp(ContextAllocator));
					}
				}
			}

			FCollisionContextAllocator* GetContextAllocator(const int32 Index)
			{
				return ContextAllocators[Index].Get();
			}
		
			/**
			 * @brief The set of collision constraints for the current tick (created or reinstated)
			 * 
			 * @note Some elements may be null (constraints that have been deleted)
			 * @note This is not thread-safe and should not be used during the collision detection phase (i.e., when the list is being built)
			*/
			TArrayView<FPBDCollisionConstraint* const> GetConstraints() const
			{ 
				return MakeArrayView(ActiveConstraints);
			}

			/**
			 * @brief The set of sweep collision constraints for the current tick (created or reinstated)
			 * 
			 * @note Some elements may be null (constraints that have been explicitly deleted)
			 * @note This is not thread-safe and should not be used during the collision detection phase (i.e., when the list is being built)
			*/
			TArrayView<FPBDCollisionConstraint* const> GetCCDConstraints() const
			{ 
				return MakeArrayView(ActiveCCDConstraints);
			}

			/**
			 * @brief The set of collision constraints for the current tick (created or reinstated)
			 * 
			 * @note Some elements may be null (constraints that have been explicitly deleted)
			 * @note This is not thread-safe and should not be used during the collision detection phase (i.e., when the list is being built)
			 */
			TArrayView<const FPBDCollisionConstraint* const> GetConstConstraints() const
			{
				return MakeArrayView(ActiveConstraints);
			}

			int32 GetCurrentEpoch() const
			{
				return CurrentEpoch;
			}

			/**
			* Has the constraint expired. An expired constraint is one that was not refreshed this tick.
			* 
			* @note Sleeping constrainst will report they are expired, but they should not be deleted until awoken.
			* We deliberately don't check the sleeping flag here because it adds a cache miss. This function
			* should be called only when you already know the sleep state (or you must also check IsSleeping())
			*/
			bool IsConstraintExpired(const FPBDCollisionConstraint& Constraint)
			{
				const bool bIsExpired = Constraint.GetContainerCookie().LastUsedEpoch < CurrentEpoch;
				return bIsExpired;
			}

			/**
			 * @brief Destroy all constraints
			*/
			void Reset()
			{
				ActiveConstraints.Reset();
				ActiveCCDConstraints.Reset();
				ParticlePairMidPhases.Reset();

				// NOTE: this deletes all the storage for constraints and midphases - must be last
				ContextAllocators.Reset();
			}

			/**
			 * @brief Called at the start of the frame to clear the frame's active collision list.
			 * @todo(chaos): This is only required because of the way events work (see AdvanceOneTimeStepTask::DoWork)
			*/
			void BeginFrame()
			{
				ResetActiveConstraints();

				ActiveConstraints.Reset();
				ActiveCCDConstraints.Reset();
			}

			/**
			 * @brief Called at the start of the tick to prepare for collision detection.
			 * Resets the list of active contacts.
			*/
			void BeginDetectCollisions()
			{
				check(!bInCollisionDetectionPhase);
				bInCollisionDetectionPhase = true;

				// Update the tick counter
				// NOTE: We do this here rather than in EndDetectionCollisions so that any contacts injected
				// before collision detection count as the previous frame's collisions, e.g., from Islands
				// that are manually awoken by modifying a particle on the game thread. This also needs to be
				// done where we reset the Constraints array so that we can tell we have a valid index from
				// the Epoch.
				++CurrentEpoch;

				for (const TUniquePtr<FCollisionContextAllocator>& ContextAllocator : ContextAllocators)
				{
					ContextAllocator->BeginDetectCollisions(CurrentEpoch);
				}

				ResetActiveConstraints();

				// Clear the collision list for this tick - we are about to rebuild them
				ActiveConstraints.Reset();
				ActiveCCDConstraints.Reset();
			}

			/**
			 * @brief Called after collision detection to clean up
			 * Prunes unused contacts
			*/
			CHAOS_API void EndDetectCollisions();

			/**
			 * @brief Called each tick after the graph is updated to remove unused collisions
			*/
			void PruneExpiredItems()
			{
				PruneExpiredMidPhases();
			}

			/**
			 * Collect all the midphases created on the context allocators (probably on multiple threads) and register them
			*/
			CHAOS_API void ProcessNewMidPhases();

			/**
			 * @brief If we add new constraints after collision detection, do what needs to be done to add them to the system
			*/
			void ProcessInjectedConstraints()
			{
				ProcessNewItems();
			}

			/**
			 * This should be called when a sleeping constraint is activated after the narrow phase, typically
			 * in the island manager. This is mainly so that any systems asking for a list of active constraints
			 * are provided all constraints that may have contributed to the simulation, including those that
			 * were added when an island is awoken.
			 */
			void AddActiveConstraint(FPBDCollisionConstraint* Constraint);

			/**
			 * @brief Add a set of pre-built constraints and build required internal mapping data
			 * This is used by the resim cache when restoring constraints after a desync
			*/
			void AddResimConstraints(const TArray<FPBDCollisionConstraint>& InConstraints, const FCollisionContext Context)
			{
				for (const FPBDCollisionConstraint& SourceConstraint : InConstraints)
				{
					// We must keep the particles in the same order that the broadphase would generate when
					// finding or creating the midphase. This is because Collision Constraints may have the
					// particles in the opposite order to the midphase tha owns them.
					FGeometryParticleHandle* Particle0 = SourceConstraint.Particle[0];
					FGeometryParticleHandle* Particle1 = SourceConstraint.Particle[1];
					if (ShouldSwapParticleOrder(Particle0, Particle1))
					{
						Swap(Particle0, Particle1);
					}

					FParticlePairMidPhase* MidPhase = Context.GetAllocator()->GetMidPhase(Particle0, Particle1, SourceConstraint.Particle[0], Context);

					// We may be adding multiple constraints for the same particle pair, so we need
					// to make sure the map is up to date in the case where we just created a new MidPhase
					ProcessNewMidPhases();
				
					if (MidPhase != nullptr)
					{
						MidPhase->InjectCollision(SourceConstraint, Context);
					}
				}

				ProcessNewConstraints();
			}

			/**
			* @brief Sort all the constraints for better solver stability
			*/
			void SortConstraintsHandles()
			{
				SortActiveConstraints();
			}

			/**
			 * @brief Destroy all collision and caches involving the particle
			 * Called when a particle is destroyed or disabled (not sleeping).
			*/
			CHAOS_API void RemoveParticle(FGeometryParticleHandle* Particle);

			/**
			 * @brief Iterate over all collisions (read-only), including sleeping ones
			 * Visitor signature: ECollisionVisitorResult(const FPBDCollisionConstraint&)
			*/
			template<typename TLambda>
			void VisitConstCollisions(const TLambda& Visitor, const ECollisionVisitorFlags& VisitFlags = ECollisionVisitorFlags::VisitDefault) const
			{
				for (const FParticlePairMidPhasePtr& MidPhase : ParticlePairMidPhases)
				{
					if (MidPhase->VisitConstCollisions(Visitor, VisitFlags) == ECollisionVisitorResult::Stop)
					{
						return;
					}
				}
			}

			/**
			 * @brief Iterate over all collisions (write-enabled), including sleeping ones
			 * Visitor signature: ECollisionVisitorResult(FPBDCollisionConstraint&)
			*/
			template<typename TLambda>
			void VisitCollisions(const TLambda& Visitor) const
			{
				for (const FParticlePairMidPhasePtr& MidPhase : ParticlePairMidPhases)
				{
					if (MidPhase->VisitCollisions(Visitor) == ECollisionVisitorResult::Stop)
					{
						return;
					}
				}
			}

			/**
			 * @brief Iterate over all midphases
			 * Visitor signature: ECollisionVisitorResult(FParticlePairMidPhase&)
			*/
			template<typename TLambda>
			void VisitMidPhases(const TLambda& Visitor)
			{
				for (FParticlePairMidPhasePtr& MidPhase : ParticlePairMidPhases)
				{
					if (Visitor(*MidPhase) == ECollisionVisitorResult::Stop)
					{
						return;
					}
				}
			}

			/**
			 * @brief Get the total number of midphases
			 */
			int32 GetNumMidPhases() const
			{
				return ParticlePairMidPhases.Num();
			}

			/**
			 * Enable/Disable determinism (extra sorting steps after collision detection)
			*/
			void SetIsDeterministic(const bool bInIsDeterministic)
			{
				bIsDeteministic = bInIsDeterministic;
			}

		private:
			friend class FCollisionContextAllocator;

			// Gather all the newly created items and finalize their setup
			void ProcessNewItems();

			void AddMidPhase(FParticlePairMidPhasePtr&& MidPhase)
			{
				ParticlePairMidPhases.Emplace(MoveTemp(MidPhase));
			}

			// Remove ParticlePairMidPhase from each Particles list of collisions
			// NOTE: One or both particles may have been destroyed in which case it will
			// have been set to null on the midphase.
			void DetachParticlePairMidPhase(FParticlePairMidPhase* MidPhase)
			{
				if (FGeometryParticleHandle* Particle0 = MidPhase->GetParticle0())
				{
					Particle0->ParticleCollisions().RemoveMidPhase(Particle0, MidPhase);
				}

				if (FGeometryParticleHandle* Particle1 = MidPhase->GetParticle1())
				{
					Particle1->ParticleCollisions().RemoveMidPhase(Particle1, MidPhase);
				}
			}

			// Find all midphases that have not been updated this tick (i.e., bounds no longer overlap) and destroy them
			CHAOS_API void PruneExpiredMidPhases();

			// Collect all the constraints activated in the collision tasks and register them (calls ActivateConstraintImp on each)
			CHAOS_API void ProcessNewConstraints();
			CHAOS_API void ProcessNewNonCcdConstraints(const TArray<FPBDCollisionConstraint*>& NewConstraints, TArray<FPBDCollisionConstraint*>& AllCCDConstraints);
			CHAOS_API void ProcessNewCcdConstraints();

			CHAOS_API void AddActiveConstraintImpl(FPBDCollisionConstraint* Constraint);
			CHAOS_API void SetConstraintCookie(FPBDCollisionConstraint* Constraint, int32 Index);
			CHAOS_API void RemoveActiveConstraint(FPBDCollisionConstraint& Constraint);

			CHAOS_API void SortActiveConstraints();

			CHAOS_API void ResetActiveConstraints();

			// The container that owns the allocator (only needed because new constraints need to know)
			FPBDCollisionConstraints* CollisionContainer;

			// Storage for collisins and midphases, one for each thread on which we detect collisions
			TArray<TUniquePtr<FCollisionContextAllocator>> ContextAllocators;
			int32 NumCollisionsPerBlock;

			// All of the overlapping particle pairs in the scene
			TArray<FParticlePairMidPhasePtr> ParticlePairMidPhases;

			// The active constraints (added or recovered this tick)
			TArray<FPBDCollisionConstraint*> ActiveConstraints;

			// The active sweep constraints (added or recovered this tick)
			TArray<FPBDCollisionConstraint*> ActiveCCDConstraints;

			// The current epoch used to track out-of-date contacts. A constraint whose Epoch is
			// older than the current Epoch at the end of the tick was not refreshed this tick.
			int32 CurrentEpoch;

			// Whether we running a deterministic sim
			bool bIsDeteministic;

			// For assertions
			bool bInCollisionDetectionPhase;
		};

	}	// namespace Private

	///////////////////////////////////////////////////////////////////////////////////////////////
	///////////////////////////////////////////////////////////////////////////////////////////////
	//
	// Below here is for code that is moved to avoid circular header dependencies
	// See ParticlePairMidPhase.h and ParticleCollisions.h
	//
	///////////////////////////////////////////////////////////////////////////////////////////////
	///////////////////////////////////////////////////////////////////////////////////////////////

	// Check the visitor flags and return true if we want to visit the constraint
	inline bool CollisionVisitorShouldVisit(const FPBDCollisionConstraint* Constraint, const ECollisionVisitorFlags VisitFlags)
	{
		if (Constraint == nullptr)
		{
			return false;
		}

		if (!(VisitFlags & ECollisionVisitorFlags::VisitDisabled))
		{
			// Awkward. Sleeping constraints are considered "disabled" so if we want to
			// visit sleeping constraints we must ignore the disabled state when asleep
			if (!(VisitFlags & ECollisionVisitorFlags::VisitSleeping))
			{
				// We do NOT want to visit sleeping, so we can just check the enabled flag
				if (!Constraint->IsEnabled())
				{
					return false;
				}
			}
			else
			{
				// We do want to visit sleeping, so we only skip disabled-awake constraints
				if (!Constraint->IsEnabled() && !Constraint->IsSleeping())
				{
					return false;
				}
			}
		}

		if (!(VisitFlags & ECollisionVisitorFlags::VisitSleeping))
		{
			if (Constraint->IsSleeping())
			{
				return false;
			}
		}

		if (!(VisitFlags & ECollisionVisitorFlags::VisitExpired))
		{
			if (!Constraint->IsCurrent())
			{
				return false;
			}
		}

		return true;
	}

	template<typename TLambda>
	inline ECollisionVisitorResult FParticlePairMidPhase::VisitCollisions(const TLambda& Visitor, const ECollisionVisitorFlags VisitFlags)
	{
		if (GetMidPhaseType() == EParticlePairMidPhaseType::ShapePair)
		{
			FShapePairParticlePairMidPhase* This = static_cast<FShapePairParticlePairMidPhase*>(this);

			for (FSingleShapePairCollisionDetector& ShapePair : This->ShapePairDetectors)
			{
				if (CollisionVisitorShouldVisit(ShapePair.GetConstraint(), VisitFlags))
				{
					if (Visitor(*ShapePair.GetConstraint()) == ECollisionVisitorResult::Stop)
					{
						return ECollisionVisitorResult::Stop;
					}
				}
			}
		}

		if (GetMidPhaseType() == EParticlePairMidPhaseType::Generic)
		{
			FGenericParticlePairMidPhase* This = static_cast<FGenericParticlePairMidPhase*>(this);
			for (auto& KVP : This->Constraints)
			{
				FPBDCollisionConstraintPtr& Constraint = KVP.Value;

				if (CollisionVisitorShouldVisit(Constraint.Get(), VisitFlags))
				{
					if (Visitor(*Constraint) == ECollisionVisitorResult::Stop)
					{
						return ECollisionVisitorResult::Stop;
					}
				}
			}
		}

		if (GetMidPhaseType() == EParticlePairMidPhaseType::SphereApproximation)
		{
			FSphereApproximationParticlePairMidPhase* This = static_cast<FSphereApproximationParticlePairMidPhase*>(this);
			FPBDCollisionConstraint* Constraint = This->Constraint.Get();
			if (Constraint != nullptr)
			{
				if (CollisionVisitorShouldVisit(Constraint, VisitFlags))
				{
					if (Visitor(*Constraint) == ECollisionVisitorResult::Stop)
					{
						return ECollisionVisitorResult::Stop;
					}
				}
			}
		}

		return ECollisionVisitorResult::Continue;
	}


	template<typename TLambda>
	inline ECollisionVisitorResult FParticlePairMidPhase::VisitConstCollisions(const TLambda& Visitor, const ECollisionVisitorFlags VisitFlags) const
	{
		if (GetMidPhaseType() == EParticlePairMidPhaseType::ShapePair)
		{
			const FShapePairParticlePairMidPhase* This = static_cast<const FShapePairParticlePairMidPhase*>(this);
			for (const FSingleShapePairCollisionDetector& ShapePair : This->ShapePairDetectors)
			{
				if (CollisionVisitorShouldVisit(ShapePair.GetConstraint(), VisitFlags))
				{
					if (Visitor(*ShapePair.GetConstraint()) == ECollisionVisitorResult::Stop)
					{
						return ECollisionVisitorResult::Stop;
					}
				}
			}
		}

		if (GetMidPhaseType() == EParticlePairMidPhaseType::Generic)
		{
			const FGenericParticlePairMidPhase* This = static_cast<const FGenericParticlePairMidPhase*>(this);
			for (const auto& KVP : This->Constraints)
			{
				const FPBDCollisionConstraintPtr& Constraint = KVP.Value;
				
				if (CollisionVisitorShouldVisit(Constraint.Get(), VisitFlags))
				{
					if (Visitor(*Constraint) == ECollisionVisitorResult::Stop)
					{
						return ECollisionVisitorResult::Stop;
					}
				}
			}
		}

		if (GetMidPhaseType() == EParticlePairMidPhaseType::SphereApproximation)
		{
			const FSphereApproximationParticlePairMidPhase* This = static_cast<const FSphereApproximationParticlePairMidPhase*>(this);
			const FPBDCollisionConstraint* Constraint = This->Constraint.Get();
			if (Constraint != nullptr)
			{
				if (CollisionVisitorShouldVisit(Constraint, VisitFlags))
				{
					if (Visitor(*Constraint) == ECollisionVisitorResult::Stop)
					{
						return ECollisionVisitorResult::Stop;
					}
				}
			}
		}

		return ECollisionVisitorResult::Continue;
	}


	template<typename TLambda>
	inline ECollisionVisitorResult FParticleCollisions::VisitMidPhases(const TLambda& Lambda)
	{
		for (int32 Index = 0; Index < MidPhases.Num(); ++Index)
		{
			if (Lambda(*MidPhases[Index].Value) == ECollisionVisitorResult::Stop)
			{
				return ECollisionVisitorResult::Stop;
			}
		}
		return ECollisionVisitorResult::Continue;
	}

	template<typename TLambda>
	inline ECollisionVisitorResult FParticleCollisions::VisitConstMidPhases(const TLambda& Lambda) const
	{
		for (int32 Index = 0; Index < MidPhases.Num(); ++Index)
		{
			if (Lambda(*MidPhases[Index].Value) == ECollisionVisitorResult::Stop)
			{
				return ECollisionVisitorResult::Stop;
			}
		}
		return ECollisionVisitorResult::Continue;
	}

	template<typename TLambda>
	inline ECollisionVisitorResult FParticleCollisions::VisitCollisions(const TLambda& Visitor, const ECollisionVisitorFlags VisitFlags)
	{
		return VisitMidPhases([&Visitor, VisitFlags](FParticlePairMidPhase& MidPhase)
			{
				if (MidPhase.VisitCollisions(Visitor, VisitFlags) == ECollisionVisitorResult::Stop)
				{
					return ECollisionVisitorResult::Stop;
				}
				return ECollisionVisitorResult::Continue;
			});
	}

	template<typename TLambda>
	inline ECollisionVisitorResult FParticleCollisions::VisitConstCollisions(const TLambda& Visitor, const ECollisionVisitorFlags VisitFlags) const
	{
		return VisitConstMidPhases([&Visitor, VisitFlags](const FParticlePairMidPhase& MidPhase)
			{
				if (MidPhase.VisitConstCollisions(Visitor, VisitFlags) == ECollisionVisitorResult::Stop)
				{
					return ECollisionVisitorResult::Stop;
				}
				return ECollisionVisitorResult::Continue;
			});
	}


}
