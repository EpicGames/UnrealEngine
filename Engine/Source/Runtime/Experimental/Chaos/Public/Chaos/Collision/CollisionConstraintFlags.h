// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/Framework/MultiBufferResource.h"
#include "Chaos/GeometryParticlesfwd.h"
#include "Chaos/ParticleHandleFwd.h"
#include "Chaos/PBDRigidsEvolutionFwd.h"
#include "Containers/SpscQueue.h"

#ifndef WITH_TODO_COLLISION_DISABLE
#define WITH_TODO_COLLISION_DISABLE 0
#endif

namespace Chaos
{
	/**
	 * @brief Flags for user-control over per-particle collision behaviour
	 * @todo(chaos): move SmoothEdgeCollisions to FRigidParticleControlFlags and remove ECollisionConstraintFlags
	*/
	enum class ECollisionConstraintFlags : uint32
	{
		CCF_None                       = 0x0,
		CCF_BroadPhaseIgnoreCollisions = 0x1,	// Not used
		CCF_SmoothEdgeCollisions       = 0x2,
		CCF_DummyFlag
	};

	// @todo(chaos): this should be broken out into FIgnoreCollisionManager (PT-only) and FIgnoreCollisionManagerProxy (interop).
	// E.g., The interop could store sets of particle proxy pairs to enable/disable and call the ParticleHandle API on the PT object. 
	// Or it could use the solver to map IDs to particle handles like we currently do.
	class FIgnoreCollisionManager
	{
	public:
		using FHandleID = FUniqueIdx;
		using FDeactivationSet = TSet<FUniqueIdx>;
		using FActiveMap UE_DEPRECATED(5.1, "This type is no longer used") = TMap<FHandleID, TArray<FHandleID>>;
		using FPendingMap = TMap<FHandleID, TArray<FHandleID>>;

		struct FStorageData
		{
			FPendingMap PendingActivations;
			FDeactivationSet PendingDeactivations;
			int32 ExternalTimestamp = INDEX_NONE;

			void Reset()
			{
				PendingActivations.Reset();
				PendingDeactivations.Reset();
				ExternalTimestamp = INDEX_NONE;
			}
		};

		FIgnoreCollisionManager()
			: StorageDataProducer(nullptr)
		{
			StorageDataProducer = GetNewStorageData();
		}

		//
		//
		// PHYSICS THREAD API
		//
		//

		CHAOS_API bool ContainsHandle(FHandleID Body0) const;

		CHAOS_API bool IgnoresCollision(FHandleID Body0, FHandleID Body1) const;

		CHAOS_API int32 NumIgnoredCollision(FHandleID Body0) const;

		/**
		 * Return true if we should ignore collisions between the two particles
		 */
		CHAOS_API bool IgnoresCollision(const FGeometryParticleHandle* Particle0, const FGeometryParticleHandle* Particle1) const;

		/**
		 * Add an ignore entry for collisions of Particle0->Particle1.
		 * Note, this will add an entry for both particles against the other if both particles are dynamic or kinematic.
		 */
		CHAOS_API void AddIgnoreCollisions(FGeometryParticleHandle* Particle0, FGeometryParticleHandle* Particle1);

		/**
		 * Remove an ignore entry for collisions of Particle0->Particle1 (in both directions if applicable)
		 */
		CHAOS_API void RemoveIgnoreCollisions(FGeometryParticleHandle* Particle0, FGeometryParticleHandle* Particle1);

		//
		//
		// GAME THREAD API
		//
		//

		FPendingMap& GetPendingActivationsForGameThread(int32 ExternalTimestamp) 
		{
			if (StorageDataProducer->ExternalTimestamp == INDEX_NONE)
			{
				StorageDataProducer->ExternalTimestamp = ExternalTimestamp;
			}
			else
			{
				ensure(StorageDataProducer->ExternalTimestamp == ExternalTimestamp);
			}

			return StorageDataProducer->PendingActivations;
		}

		FDeactivationSet& GetPendingDeactivationsForGameThread(int32 ExternalTimestamp)
		{
			if (StorageDataProducer->ExternalTimestamp == INDEX_NONE)
			{
				StorageDataProducer->ExternalTimestamp = ExternalTimestamp;
			}
			else
			{
				ensure(StorageDataProducer->ExternalTimestamp == ExternalTimestamp);
			}

			return StorageDataProducer->PendingDeactivations;
		}

		void PushProducerStorageData_External(int32 ExternalTimestamp)
		{
			if (StorageDataProducer->ExternalTimestamp != INDEX_NONE)
			{
				ensure(ExternalTimestamp == StorageDataProducer->ExternalTimestamp);
				StorageDataQueue.Enqueue(StorageDataProducer);
				StorageDataProducer = GetNewStorageData();
			}
		}

		//
		//
		// INTEROP API
		//
		//

		/*
		*
		*/
		CHAOS_API void ProcessPendingQueues(FPBDRigidsSolver& Solver);

		/*
		*
		*/
		CHAOS_API void PopStorageData_Internal(int32 ExternalTimestamp);


		// DEPRECATED
		UE_DEPRECATED(5.2, "The physics thread uses the ParticleHandle API. See AddIgnoreCollisions")
		void AddIgnoreCollisionsFor(FHandleID Body0, FHandleID Body1) { }
		UE_DEPRECATED(5.2, "The physics thread uses the ParticleHandle API. See RemoveIgnoreCollisions")
		int32 RemoveIgnoreCollisionsFor(FHandleID Body0, FHandleID Body1) { return 0; }

	private:

		CHAOS_API FGeometryParticleHandle* GetParticleHandle(FHandleID Body, FPBDRigidsSolver& Solver);
		CHAOS_API void SetIgnoreCollisionFlag(FPBDRigidParticleHandle* Rigid, const bool bUsesIgnoreCollisionManager);
		CHAOS_API void AddIgnoreCollisionsImpl(FHandleID Body0, FHandleID Body1);
		CHAOS_API int32 RemoveIgnoreCollisionsImpl(FHandleID Body0, FHandleID Body1);

		// Multiple sources can request an ignore pair, so we handle a simple count of
		// ignore requests to ensure we don't prematurely remove an ignore pair
		// #CHAOSTODO replace bi-directional map to pair hash to reduce required storage and dependency
		struct FIgnoreEntry
		{
			FIgnoreEntry() = delete;
			explicit FIgnoreEntry(FHandleID InId)
				: Id(InId)
				, Count(1) // Begin at 1 for convenience when adding entries
			{}

			FHandleID Id;
			int32 Count;
		};

		FStorageData* GetNewStorageData()
		{
			FStorageData* StorageData;
			if (StorageDataFreePool.Dequeue(StorageData))
			{
				return StorageData;
			}

			StorageDataBackingBuffer.Emplace(MakeUnique<FStorageData>());
			return StorageDataBackingBuffer.Last().Get();
		}

		void ReleaseStorageData(FStorageData *InStorageData)
		{
			InStorageData->Reset();
			StorageDataFreePool.Enqueue(InStorageData);
		}

		// Maps collision body0 to a list of other bodies and a count for how many
		// sources have requested for this pair to be ignored
		TMap<FHandleID, TArray<FIgnoreEntry>> IgnoreCollisionsList;

		FPendingMap PendingActivations;
		FDeactivationSet PendingDeactivations;

		// Producer storage data, pending changes written here until pushed into queue.
		FStorageData* StorageDataProducer;

		TSpscQueue<FStorageData*> StorageDataQueue; // Queue of storage data being passed to physics thread
		TSpscQueue<FStorageData*> StorageDataFreePool;	//free pool of storage data
		TArray<TUniquePtr<FStorageData>> StorageDataBackingBuffer;	// Holds unique ptrs for storage data allocation
	};

} // Chaos
