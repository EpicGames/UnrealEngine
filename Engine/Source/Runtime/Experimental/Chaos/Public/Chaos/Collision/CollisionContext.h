// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/ChaosDebugName.h"
#include "Chaos/Core.h"

namespace Chaos
{
	namespace Private
	{
		class FCollisionContextAllocator;
	}

	class FParticlePairMidPhase;

	class FCollisionDetectorSettings
	{
	public:
		FCollisionDetectorSettings()
			: BoundsExpansion(0)
			, BoundsVelocityInflation(0)
			, MaxVelocityBoundsExpansion(0)
			, BoundsVelocityInflationMACD(0)
			, MaxVelocityBoundsExpansionMACD(0)
			, bFilteringEnabled(true)
			, bDeferNarrowPhase(false)
			, bAllowManifolds(true)
			, bAllowManifoldReuse(true)
			, bAllowCCD(true)
			, bAllowMACD(false)
			, EvolutionName()
		{
		}

		// Shape bounds are expanded by this in all 3 directions. This is also used as the CullDistance for collision response.
		FReal BoundsExpansion;
		
		// Shape bounds in the broadphase are expanded by this multiple of velocity (normal collision mode)
		FReal BoundsVelocityInflation;

		// We only allow the bounds to grow from velocity by this much (normal collision mode)
		FReal MaxVelocityBoundsExpansion;

		// Shape bounds in the broadphase are expanded by this multiple of velocity (movement-aware collision mode)
		FReal BoundsVelocityInflationMACD;

		// We only allow the bounds to grow from velocity by this much (movement-aware collision mode)
		FReal MaxVelocityBoundsExpansionMACD;

		// Whether to check the shape query flags in the narrow phase (e.g., Rigid Body nodes have already performed filtering prior to collision detection)
		bool bFilteringEnabled;

		// Whether to defer the narrow phase to the constraint-solve phase. This is only enabled by RBAN. It is not useful for the main solver because 
		// we would not know the contact details when we call the collision modifier callbacks. It is used by RBAN to allow us to run 1 joint iteration 
		// prior to collision detection which gives better results.
		bool bDeferNarrowPhase;

		// Whether to use one-shot manifolds where supported
		bool bAllowManifolds;

		// Whether we can reuse manifolds between frames if contacts have not moved far
		bool bAllowManifoldReuse;

		// Whether CCD is allowed (disabled by default for RBAN)
		bool bAllowCCD;

		// Whether MACD is allowed (disabled for RBAN)
		bool bAllowMACD;

		// Useful for debugging (esp. PIE client/server)
		FSharedDebugName EvolutionName;

		const FString& GetName() const
		{
			return EvolutionName.Value();
		}
	};

	/**
	 * Data passed down into the collision detection functions.
	 */
	class FCollisionContext
	{
	public:
		FCollisionContext()
			: Settings(nullptr)
			, Allocator(nullptr)
			, MidPhase(nullptr)
		{
		}

		void Reset()
		{
			Settings = nullptr;
			Allocator = nullptr;
			MidPhase = nullptr;
		}

		const FCollisionDetectorSettings& GetSettings() const { return *Settings; }
		void SetSettings(const FCollisionDetectorSettings& InSettings) { Settings = &InSettings; }

		Private::FCollisionContextAllocator* GetAllocator() const { return Allocator; }
		void SetAllocator(Private::FCollisionContextAllocator* InAllocator) { Allocator = InAllocator; }

		const FCollisionDetectorSettings* Settings;

		Private::FCollisionContextAllocator* Allocator;

		// This is used in the older collision detection path which is still used for particles that do not flatten their implicit hierrarchies
		// into the Particle's ShapesArray. Currently this is only Clusters.
		// @todo(chaos): remove thsi from here and make it a parameter on ConstructCollisions and all inner functions.
		FParticlePairMidPhase* MidPhase;
	};
}