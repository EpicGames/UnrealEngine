// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Collision/CollisionContext.h"

#include "ChaosStats.h"

namespace Chaos
{
	class FPBDCollisionConstraints;
	class FEvolutionResimCache;

	class FCollisionDetector
	{
	public:
		FCollisionDetector(FPBDCollisionConstraints& InCollisionContainer)
			: CollisionContainer(InCollisionContainer)
		{
		}

		virtual ~FCollisionDetector() = default;

		UE_DEPRECATED(5.2, "Moved to FPBDCollisionConstraints")
		CHAOS_API const FCollisionDetectorSettings& GetSettings() const;

		UE_DEPRECATED(5.2, "Moved to FPBDCollisionConstraints")
		CHAOS_API void SetSettings(const FCollisionDetectorSettings& InSettings);

		UE_DEPRECATED(5.2, "Moved to FPBDCollisionConstraints and renamed to SetCullDistance")
		CHAOS_API void SetBoundsExpansion(const FReal InBoundsExpansion);

		UE_DEPRECATED(5.2, "No longer supported")
		void SetBoundsVelocityInflation(const FReal InBoundsVelocityInflation) {}

		FPBDCollisionConstraints& GetCollisionContainer() { return CollisionContainer; }

		virtual void DetectCollisions(const FReal Dt, FEvolutionResimCache* ResimCache) = 0;

	protected:
		FPBDCollisionConstraints& CollisionContainer;
	};

}
