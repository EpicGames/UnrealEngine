// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// HEADER_UNIT_SKIP - Internal

#include "Chaos/Core.h"
#include "Chaos/Collision/CollisionConstraintFlags.h"
#include "Chaos/Collision/CollisionFilterBits.h"
#include "Chaos/CollisionFilterData.h"
#include "Chaos/GeometryParticles.h"
#include "Chaos/ImplicitObjectType.h"
#include "Chaos/ParticleHandle.h"

typedef uint8 FMaskFilter;

namespace Chaos
{
	/**
	 * Check whether the two particles need to be considered in the broadphase.
	 * @return true if the particle pair should be considered by the broadphase
	 */
	inline bool ParticlePairBroadPhaseFilter(
		const FGeometryParticleHandle* Particle1,
		const FGeometryParticleHandle* Particle2,
		const FIgnoreCollisionManager* IgnoreCollisionManager)
	{
		if (Particle1 == Particle2)
		{
			return false;
		}

		bool bIsKinematic1 = true;
		bool bUseIgnoreCollisionManager1 = false;
		bool bDisabled1 = false;
		int32 CollisionGroup1 = 0;
		const FPBDRigidParticleHandle* Rigid1 = Particle1->CastToRigidParticle();
		if (Rigid1 != nullptr)
		{
			bIsKinematic1 = Rigid1->IsKinematic();
			bUseIgnoreCollisionManager1 = Rigid1->UseIgnoreCollisionManager();
			bDisabled1 = Rigid1->Disabled();
			CollisionGroup1 = Rigid1->CollisionGroup();
		}

		bool bIsKinematic2 = true;
		bool bUseIgnoreCollisionManager2 = false;
		bool bDisabled2 = false;
		int32 CollisionGroup2 = 0;
		const FPBDRigidParticleHandle* Rigid2 = Particle2->CastToRigidParticle();
		if (Rigid2 != nullptr)
		{
			bIsKinematic2 = Rigid2->IsKinematic();
			bUseIgnoreCollisionManager2 = Rigid2->UseIgnoreCollisionManager();
			bDisabled2 = Rigid2->Disabled();
			CollisionGroup2 = Rigid2->CollisionGroup();
		}

		// @todo(chaos): This should not be happening if the disabled particles are removed from the active particles list, but GeometryCollection may leave them there
		//check(!bDisabled2);
		if (bDisabled1 || bDisabled2)
		{
			return false;
		}

		// At least one particle needs to be dynamic to generate a collision response
		if (bIsKinematic1 && bIsKinematic2)
		{
			return false;
		}
		check((Rigid1 != nullptr) || (Rigid2 != nullptr));

		// CollisionGroups are used by geometry collections for high-level collision filtering
		// CollisionGroup == 0 : Collide_With_Everything
		// CollisionGroup == INDEX_NONE : Disabled collisions
		// CollisionGroup1 != CollisionGroup2 : Disabled collisions (if other conditions not met)
		if (CollisionGroup1 == INDEX_NONE || CollisionGroup2 == INDEX_NONE)
		{
			return false;
		}
		if ((CollisionGroup1 != 0) && (CollisionGroup2 != 0) && (CollisionGroup1 != CollisionGroup2))
		{
			return false;
		}

		// Is this particle interaction governed by the IgnoreCollisionManager? If so, check to see if interaction is allowed
		if ((bUseIgnoreCollisionManager1 || bUseIgnoreCollisionManager2) && (IgnoreCollisionManager != nullptr))
		{
			if (IgnoreCollisionManager->IgnoresCollision(Particle1, Particle2))
			{
				return false;
			}
		}

		return true;
	}


	/**
	* A filter set to all zeroes means we do not filter
	*/
	UE_DEPRECATED(5.8, "Use FShapeFilterData.IsValid instead")
	inline bool IsFilterValid(const FCollisionFilterData& Filter)
	{
		return Filter.Word0 || Filter.Word1 || Filter.Word2 || Filter.Word3;
	}

	/**
	* Does the shape collide with anything at all in the Sim? (else it is query-only)
	*/
	inline bool FilterHasSimEnabled(const FPerShapeData* Shape)
	{
		return (!Shape || (Shape->GetSimEnabled() && Shape->GetShapeFilterData().IsSimValid()));
	}

	inline bool DoCollide(EImplicitObjectType ImplicitType, const FPerShapeData* Shape)
	{
		//
		// Disabled shapes do not collide
		//
		if (!FilterHasSimEnabled(Shape)) return false;

		//
		// Triangle Mesh geometry is only used if the shape specifies UseComplexAsSimple
		//
		if (Shape)
		{
			if (ImplicitType == ImplicitObjectType::TriangleMesh && Shape->GetCollisionTraceType() != EChaosCollisionTraceFlag::Chaos_CTF_UseComplexAsSimple)
			{
				return false;
			}
			else if (Shape->GetCollisionTraceType() == EChaosCollisionTraceFlag::Chaos_CTF_UseComplexAsSimple && ImplicitType != ImplicitObjectType::TriangleMesh)
			{
				return false;
			}
		}
		else if (ImplicitType == ImplicitObjectType::TriangleMesh)
		{
			return false;
		}

		return true;
	}

	/**
	* Sim Collision filter (i.e., not Query collision filter)
	*/
	inline bool DoCollide(EImplicitObjectType Implicit0Type, const FPerShapeData* Shape0, EImplicitObjectType Implicit1Type, const FPerShapeData* Shape1)
	{

		if (!DoCollide(Implicit0Type, Shape0)) { return false; }
		if (!DoCollide(Implicit1Type, Shape1)) { return false; }

		//
		// Shape Filtering
		//
		if (Shape0 && Shape1)
		{
			const Filter::FShapeFilterData& ShapeFilter0 = Shape0->GetShapeFilterData();
			const Filter::FShapeFilterData& ShapeFilter1 = Shape1->GetShapeFilterData();
			const Filter::ENarrowFilterResult HitType = ShapeFilter0.NarrowFilter(ShapeFilter1);
			return HitType != Filter::ENarrowFilterResult::None;
		}

		return true;
	}

	/**
	 * Check whether the two particles need to be considered in the broadphase
	 * NOTE: Implicit0Type and Implicit1Type must be the inner type of the implicit, 
	 * with any decorators removed (scaled, instanced, transformed).
	 * @return true if the shape pair should be considered by the narrowphase
	 */
	inline bool ShapePairNarrowPhaseFilter(EImplicitObjectType Implicit0Type, const FPerShapeData* Shape0, EImplicitObjectType Implicit1Type, const FPerShapeData* Shape1)
	{
		return DoCollide(Implicit0Type, Shape0, Implicit1Type, Shape1);
	}
}