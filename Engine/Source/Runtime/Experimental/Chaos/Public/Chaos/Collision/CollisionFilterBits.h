// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

namespace Chaos
{
	enum { ChaosNumExtraFilterBits = 6 };
	enum { ChaosNumCollisionChannelBits = 5 };
}