// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Chaos/Core.h"
#include "Chaos/ParticleHandle.h"

namespace Chaos
{
	/**
	 * Combine two hashes in an order-independent way so that Hash(A, B) == Hash(B, A)
	 */
	inline uint32 OrderIndependentHashCombine(const uint32 A, const uint32 B)
	{
		if (A < B)
		{
			return ::HashCombine(A, B);
		}
		else
		{
			return ::HashCombine(B, A);
		}
	}

	/**
	 * Return true if the particles are in the preferred order (the first one has the lower ID)
	 */
	inline bool AreParticlesInPreferredOrder(const FGeometryParticleHandle* Particle0, const FGeometryParticleHandle* Particle1)
	{
		return (Particle1->ParticleID() < Particle0->ParticleID());
	}

	/**
	 * Used to order particles in a consistent way in Broadphase and Resim. Returns whether the partcile order should be reversed.
	 * We want the particle with the lower ID first, unless only one is dynamic in which case that one is first.
	 */
	inline bool ShouldSwapParticleOrder(const bool bIsDynamicOrSleeping0, const bool bIsDynamicOrSleeping1, const bool bIsParticle0Preferred)
	{
		return !bIsDynamicOrSleeping0 || (bIsDynamicOrSleeping1 && !bIsParticle0Preferred);
	}
	inline bool ShouldSwapParticleOrder(const FGeometryParticleHandle* Particle0, const FGeometryParticleHandle* Particle1)
	{
		const EObjectStateType ObjectState0 = Particle0->ObjectState();
		const EObjectStateType ObjectState1 = Particle1->ObjectState();
		const bool bIsDynamicOrSleeping0 = (ObjectState0 == EObjectStateType::Dynamic) || (ObjectState0 == EObjectStateType::Sleeping);
		const bool bIsDynamicOrSleeping1 = (ObjectState1 == EObjectStateType::Dynamic) || (ObjectState1 == EObjectStateType::Sleeping);
		return ShouldSwapParticleOrder(bIsDynamicOrSleeping0, bIsDynamicOrSleeping1, AreParticlesInPreferredOrder(Particle0, Particle1));
	}

	namespace Private
	{

		/**
		 * A key which uniquely identifes a particle pair for use by the collision detection system.
		 * This key can optionally be the same if particles order is reversed (see bSymmetric constructor parameter).
		 * @note This uses ParticleID and truncates it to 31 bits (from 32)
		 */
		class FCollisionParticlePairKey
		{
		public:
			using KeyType = uint64;

			FCollisionParticlePairKey()
			{
				Key.Key64 = 0;
			}

			FCollisionParticlePairKey(const FGeometryParticleHandle* Particle0, const FGeometryParticleHandle* Particle1, const bool bSymmetric = true)
			{
				GenerateKey(Particle0, Particle1, bSymmetric);
			}

			uint64 GetKey() const
			{
				return Key.Key64;
			}

			uint32 GetHash() const
			{
				// NOTE: GetTypeHash(uint64 V) does not work well if the 64 bit int is actually 2x32 bit ints
				// so we hash-combine the two 32 bit ints instead
				return ::HashCombineFast(uint32(Key.Key64 & 0xFFFFFFFF), uint32((Key.Key64 >> 32) & 0xFFFFFFFF));
			}

			friend bool operator==(const FCollisionParticlePairKey& L, const FCollisionParticlePairKey& R)
			{
				return L.Key.Key64 == R.Key.Key64;
			}

			friend bool operator!=(const FCollisionParticlePairKey& L, const FCollisionParticlePairKey& R)
			{
				return L.Key.Key64 != R.Key.Key64;
			}

			friend bool operator<(const FCollisionParticlePairKey& L, const FCollisionParticlePairKey& R)
			{
				return L.Key.Key64 < R.Key.Key64;
			}

		private:
			void GenerateKey(const FGeometryParticleHandle* Particle0, const FGeometryParticleHandle* Particle1, const bool bSymmetric)
			{
				Key.Key64 = 0;
				if ((Particle0 != nullptr) && (Particle1 != nullptr))
				{
					const bool bIsLocalID0 = Particle0->ParticleID().LocalID != INDEX_NONE;
					const bool bIsLocalID1 = Particle1->ParticleID().LocalID != INDEX_NONE;
					const uint32 ID0 = uint32((bIsLocalID0) ? Particle0->ParticleID().LocalID : Particle0->ParticleID().GlobalID);
					const uint32 ID1 = uint32((bIsLocalID1) ? Particle1->ParticleID().LocalID : Particle1->ParticleID().GlobalID);

					// If we want Key(A,B) == Key(B,A) swap the order if particle IDs to guarantee this
					if (!bSymmetric || (ID0 < ID1))
					{
						Key.Key32s[0].Key31 = ID0;
						Key.Key32s[0].IsLocal = bIsLocalID0;
						Key.Key32s[1].Key31 = ID1;
						Key.Key32s[1].IsLocal = bIsLocalID1;
					}
					else
					{
						Key.Key32s[0].Key31 = ID1;
						Key.Key32s[0].IsLocal = bIsLocalID1;
						Key.Key32s[1].Key31 = ID0;
						Key.Key32s[1].IsLocal = bIsLocalID0;
					}
				}
			}

			struct FParticleIDKey
			{
				uint32 Key31 : 31;
				uint32 IsLocal : 1;
			};
			union FIDKey
			{
				uint64 Key64;
				FParticleIDKey Key32s[2];
			};

			// This class is sensitive to changes in FParticleID - try to catch that here...
			static_assert(sizeof(FParticleID) == 8, "FParticleID size does not match FCollisionParticlePairKey (expected 64 bits)");
			static_assert(sizeof(FParticleID::GlobalID) == 4, "FParticleID::GlobalID size does not match FCollisionParticlePairKey (expected 32 bits)");
			static_assert(sizeof(FParticleID::LocalID) == 4, "FParticleID::LocalID size does not match FCollisionParticlePairKey (expected 32 bits)");
			static_assert(sizeof(FParticleIDKey) == 4, "FCollisionParticlePairKey::FParticleIDKey size is not 32 bits");
			static_assert(sizeof(FIDKey) == 8, "FCollisionParticlePairKey::FIDKey size is not 64 bits");

			FIDKey Key;
		};


		/**
		 * A key for two shapes within a particle pair. This key is will be unique for the shape pair in the particle pair, 
		 * but not will be duplicated in different particle pairs.
		 */
		class FCollisionShapePairKey
		{
		public:
			FCollisionShapePairKey()
				: Key(0)
			{
			}

			FCollisionShapePairKey(const int32 InShapeID0, const int32 InShapeID1)
				: ShapeID0(InShapeID0)
				, ShapeID1(InShapeID1)
			{
			}

			uint64 GetKey() const
			{
				return Key;
			}

			friend bool operator==(const FCollisionShapePairKey& L, const FCollisionShapePairKey& R)
			{
				return L.Key == R.Key;
			}

			friend bool operator!=(const FCollisionShapePairKey& L, const FCollisionShapePairKey& R)
			{
				return L.Key != R.Key;
			}

			friend bool operator<(const FCollisionShapePairKey& L, const FCollisionShapePairKey& R)
			{
				return L.Key < R.Key;
			}

		private:
			union
			{
				struct
				{
					uint32 ShapeID0;
					uint32 ShapeID1;
				};
				uint64 Key;
			};
		};

		/**
		 * A key used to sort collisions. This to ensure consistent collision solver order when collisions are generated 
		 * in a semi-random order (during multi-threaded collision detection).
		 * NOTE: This version does not result in collisions being sorted by Particle, but it is smaller than FCollisionSortKeyNonHashed
		 * 
		 * @todo(chaos): Hashing the particle pair key does not work well at all. Do not use this for the time being (see FCollisionSortKeyNonHashed)
		 * 
		 */
		class FCollisionSortKeyHashed
		{
		public:
			FCollisionSortKeyHashed()
				: Key(0)
			{
			}

			FCollisionSortKeyHashed(
				const FGeometryParticleHandle* InParticle0, const int32 InShapeID0,
				const FGeometryParticleHandle* InParticle1, const int32 InShapeID1)
			{
				check(false);	// Don't use unless we can make a better hash
				FCollisionParticlePairKey ParticlePairKey = FCollisionParticlePairKey(InParticle0, InParticle1, false);
				ParticlesKey = ParticlePairKey.GetHash();
				ShapesKey = ::HashCombineFast(::GetTypeHash(InShapeID0), ::GetTypeHash(InShapeID1));
			}

			friend bool operator==(const FCollisionSortKeyHashed& L, const FCollisionSortKeyHashed& R)
			{
				return L.Key == R.Key;
			}

			friend bool operator!=(const FCollisionSortKeyHashed& L, const FCollisionSortKeyHashed& R)
			{
				return L.Key != R.Key;
			}

			friend bool operator<(const FCollisionSortKeyHashed& L, const FCollisionSortKeyHashed& R)
			{
				return L.Key < R.Key;
			}

		private:
			union
			{
				struct
				{
					uint32 ParticlesKey;
					uint32 ShapesKey;
				};
				uint64 Key;
			};
		};

		/**
		 * A key used to sort collisions. This to ensure consistent collision solver order when collisions are generated
		 * in a semi-random order (during multi-threaded collision detection).
		 * NOTE: This version also results in collisions being sorted by Particle (as opposed to FCollisionSortKeyHashed)
		 * but it is larger and the comparison operator is more complex, so sorting cost is higher
		 */
		class FCollisionSortKeyNonHashed
		{
		public:
			FCollisionSortKeyNonHashed()
				: ParticlePairKey()
				, ShapePairKey()
			{
			}

			FCollisionSortKeyNonHashed(
				const FGeometryParticleHandle* InParticle0, const int32 InShapeID0,
				const FGeometryParticleHandle* InParticle1, const int32 InShapeID1)
				: ParticlePairKey(InParticle0, InParticle1, false)
				, ShapePairKey(InShapeID0, InShapeID1)
			{
			}

			friend bool operator==(const FCollisionSortKeyNonHashed& L, const FCollisionSortKeyNonHashed& R)
			{
				return (L.ParticlePairKey == R.ParticlePairKey) && (L.ShapePairKey == R.ShapePairKey);
			}

			friend bool operator!=(const FCollisionSortKeyNonHashed& L, const FCollisionSortKeyNonHashed& R)
			{
				return (L.ParticlePairKey != R.ParticlePairKey) || (L.ShapePairKey != R.ShapePairKey);
			}

			friend bool operator<(const FCollisionSortKeyNonHashed& L, const FCollisionSortKeyNonHashed& R)
			{
				// @todo(chaos): make sure this doesn't add extra branches
				const bool ParticlePairLess = (L.ParticlePairKey < R.ParticlePairKey);
				const bool ShapePairLess = (L.ShapePairKey < R.ShapePairKey);
				const bool ParticlePairNotEqual = (L.ParticlePairKey != R.ParticlePairKey);
				return ParticlePairNotEqual ? ParticlePairLess : ShapePairLess;
			}

		private:
			FCollisionParticlePairKey ParticlePairKey;
			FCollisionShapePairKey ShapePairKey;
		};

		// A key used to sort constraints
		// @todo(chaos): ideally we would use the smallest key possible here, but see comments in FCollisionSortKeyHashed
		using FCollisionSortKey = FCollisionSortKeyNonHashed;
	}


	///////////////////////////////////////////////////////////////////////////////////////////////
	///////////////////////////////////////////////////////////////////////////////////////////////
	//
	// DEPRECATED STUFF
	//
	///////////////////////////////////////////////////////////////////////////////////////////////
	///////////////////////////////////////////////////////////////////////////////////////////////


	using FCollisionParticlePairKey UE_DEPRECATED(5.4, "Not for external use") = Private::FCollisionParticlePairKey;

	class UE_DEPRECATED(5.4, "No longer used") FCollisionParticlePairConstraintKey
	{
	public:
		FCollisionParticlePairConstraintKey()
			: Key(0)
		{
		}

		UE_DEPRECATED(5.3, "Replaced with version that takes Shape and ImplicitID")
		FCollisionParticlePairConstraintKey(const FImplicitObject* Implicit0, const FBVHParticles* Simplicial0, const FImplicitObject* Implicit1, const FBVHParticles* Simplicial1)
			: Key(0)
		{
			check((Implicit0 != nullptr) || (Simplicial0 != nullptr));
			check((Implicit1 != nullptr) || (Simplicial1 != nullptr));
			GenerateHash(nullptr, Implicit0, 0, Simplicial0, nullptr, Implicit1, 0, Simplicial1);
		}

		FCollisionParticlePairConstraintKey(
			const FShapeInstance* Shape0,
			const FImplicitObject* Implicit0,
			const int32 ImplicitId0,
			const FBVHParticles* Simplicial0,
			const FShapeInstance* Shape1,
			const FImplicitObject* Implicit1,
			const int32 ImplicitId1,
			const FBVHParticles* Simplicial1)
			: Key(0)
		{
			check((Implicit0 != nullptr) || (Simplicial0 != nullptr));
			check((Implicit1 != nullptr) || (Simplicial1 != nullptr));
			GenerateHash(Shape0, Implicit0, ImplicitId0, Simplicial0, Shape1, Implicit1, ImplicitId1, Simplicial1);
		}

		uint32 GetKey() const
		{
			return Key;
		}

		friend bool operator==(const FCollisionParticlePairConstraintKey& L, const FCollisionParticlePairConstraintKey& R)
		{
			return L.Key == R.Key;
		}

		friend bool operator!=(const FCollisionParticlePairConstraintKey& L, const FCollisionParticlePairConstraintKey& R)
		{
			return !(L == R);
		}

		friend bool operator<(const FCollisionParticlePairConstraintKey& L, const FCollisionParticlePairConstraintKey& R)
		{
			return L.Key < R.Key;
		}

	private:
		void GenerateHash(
			const FShapeInstance* Shape0,
			const FImplicitObject* Implicit0,
			const int32 ImplicitId0,
			const FBVHParticles* Simplicial0,
			const FShapeInstance* Shape1,
			const FImplicitObject* Implicit1,
			const int32 ImplicitId1,
			const FBVHParticles* Simplicial1)
		{
			uint32 Hash0 = (Implicit0 != nullptr) ? ::GetTypeHash(Implicit0) : ::GetTypeHash(Simplicial0);
			Hash0 = HashCombine(Hash0, ImplicitId0);
			uint32 Hash1 = (Implicit1 != nullptr) ? ::GetTypeHash(Implicit1) : ::GetTypeHash(Simplicial1);
			Hash1 = HashCombine(Hash1, ImplicitId1);
			Key = HashCombine(Hash0, Hash1);
		}

		uint32 Key;
	};
}