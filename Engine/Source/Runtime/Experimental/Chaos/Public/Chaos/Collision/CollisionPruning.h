// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "Chaos/Core.h"
#include "Chaos/CollisionResolutionTypes.h"
#include "Chaos/Collision/PBDCollisionConstraint.h"
#include "Chaos/ParticleHandleFwd.h"

namespace Chaos
{
	/**
	 * \brief Utility to remove edge contacts on a particle if those edge contacts are "hidden" by a face contact
	 */
	class FParticleEdgeCollisionPruner
	{
	public:
		FParticleEdgeCollisionPruner(FGeometryParticleHandle* InParticle)
			: Particle(InParticle)
		{
		}

		void Prune();

	private:
		FGeometryParticleHandle* Particle;
	};

	/**
	 * \brief Utility to remove or correct edge contacts on the boundary between tweo separate meshes which coincident edges/vertices
	 * \note This only look sta collisions against triangle mesh and heightfield.
	 */
	class FParticleMeshCollisionPruner
	{
	public:
		FParticleMeshCollisionPruner(FGeometryParticleHandle* InParticle)
			: Particle(InParticle)
		{
		}

		void Prune();

	private:
		FGeometryParticleHandle* Particle;
	};

	/**
	 * \brief Utility to attempt to remove sub-surface contacts on a particle when those edge contacts are "hidden" by other contacts
	 * This is not 100% effective and may occasionally remove contacts that it shouldn't! Ideally you would just use
	 * this pruning policy in specific locations in the world where sub-surface contacts are a problem 
	 */
	class FParticleSubSurfaceCollisionPruner
	{
	public:
		FParticleSubSurfaceCollisionPruner(FGeometryParticleHandle* InParticle)
			: Particle(InParticle)
		{
		}

		void Prune(const FVec3& UpVector);

	private:
		FGeometryParticleHandle* Particle;
	};

}