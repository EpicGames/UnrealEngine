// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Collision/PBDCollisionConstraint.h"
#include "Chaos/ImplicitObject.h"
#include "Chaos/ParticleHandle.h"

namespace Chaos
{
	extern bool bChaos_Collision_EnableBoundsChecks;
	extern bool bChaos_Collision_EnableLargeMeshManifolds;

	namespace Private
	{
		// Get the implicit object type for the implicit object on the particle, taking into account the fact that the implicit
		// may be set to use a LevelSet, but only if the particle has collision particles
		inline EImplicitObjectType GetImplicitCollisionType(const FGeometryParticleHandle* Particle, const FImplicitObject* Implicit)
		{
			// NOTE: GetCollisionType(), not GetType()
			// We use CollisionType on the implicit to determine how to collide. Normally this is the same as the actual ImplicitObject's
			// type, but may be set to LevelSet in which case we will use CollisionParticles instead (if it has any).
			EImplicitObjectType ImplicitType = (Implicit != nullptr) ? GetInnerType(Implicit->GetCollisionType()) : ImplicitObjectType::Unknown;

			// If we are a levelset make sure we have CollisionParticles, otherwise go back to the builtin implicit object collision type
			if ((ImplicitType == ImplicitObjectType::LevelSet) || (ImplicitType == ImplicitObjectType::Unknown))
			{
				const FBVHParticles* Simplicial = FConstGenericParticleHandle(Particle)->CollisionParticles().Get();
				const bool bHasSimplicial = (Simplicial != nullptr) && (Simplicial->Size() > 0);
				if (!bHasSimplicial)
				{
					// NOTE: GetType(), not GetCollisionType()
					ImplicitType = (Implicit != nullptr) ? GetInnerType(Implicit->GetType()) : ImplicitObjectType::Unknown;
				}
			}

			return ImplicitType;
		}

		// Determine which bounds tests should be run for the specified implicit object pair.
		// Also sets the bIsProbe flag appropriately.
		inline FImplicitBoundsTestFlags CalculateImplicitBoundsTestFlags(
			FGeometryParticleHandle* Particle0, const FImplicitObject* Implicit0, const FPerShapeData* Shape0,
			FGeometryParticleHandle* Particle1, const FImplicitObject* Implicit1, const FPerShapeData* Shape1,
			FRealSingle& OutDistanceCheckSize)
		{
			FImplicitBoundsTestFlags Flags;

			const EImplicitObjectType ImplicitType0 = GetImplicitCollisionType(Particle0, Implicit0);
			const EImplicitObjectType ImplicitType1 = GetImplicitCollisionType(Particle1, Implicit1);
			const bool bIsSphere0 = (ImplicitType0 == ImplicitObjectType::Sphere);
			const bool bIsSphere1 = (ImplicitType1 == ImplicitObjectType::Sphere);
			const bool bIsCapsule0 = (ImplicitType0 == ImplicitObjectType::Capsule);
			const bool bIsCapsule1 = (ImplicitType1 == ImplicitObjectType::Capsule);
			const bool bIsMesh0 = (ImplicitType0 == ImplicitObjectType::TriangleMesh) || (ImplicitType0 == ImplicitObjectType::HeightField);
			const bool bIsMesh1 = (ImplicitType1 == ImplicitObjectType::TriangleMesh) || (ImplicitType1 == ImplicitObjectType::HeightField);
			const bool bIsLevelSet = ((ImplicitType0 == ImplicitObjectType::LevelSet) || (ImplicitType1 == ImplicitObjectType::LevelSet));

			const bool bHasBounds0 = (Implicit0 != nullptr) && Implicit0->HasBoundingBox();
			const bool bHasBounds1 = (Implicit1 != nullptr) && Implicit1->HasBoundingBox();
			const bool bAllowBoundsCheck = bChaos_Collision_EnableBoundsChecks && bHasBounds0 && bHasBounds1;

			Flags.bEnableAABBCheck = bAllowBoundsCheck && !(bIsSphere0 && bIsSphere1);	// No AABB test if both are spheres
			Flags.bEnableOBBCheck0 = bAllowBoundsCheck && !bIsSphere0;					// No OBB test for spheres
			Flags.bEnableOBBCheck1 = bAllowBoundsCheck && !bIsSphere1;					// No OBB test for spheres
			Flags.bEnableDistanceCheck = bAllowBoundsCheck && bIsSphere0 && bIsSphere1;	// Simple distance test for sphere pairs

			OutDistanceCheckSize = (Flags.bEnableDistanceCheck) ? FRealSingle(Implicit0->GetMarginf() + Implicit1->GetMarginf()) : 0;	// Sphere-Sphere bounds test

			// Do not try to reuse manifold points for capsules or spheres (against anything)
			// NOTE: This can also be disabled for all shape types by the solver (see GenerateCollisionImpl and the Context)
			Flags.bEnableManifoldUpdate = !bIsSphere0 && !bIsSphere1 && !bIsCapsule0 && !bIsCapsule1 && !bIsLevelSet;

			// If we don't allow large mesh manifolds we can't reuse manifolds for meshes because we may have 
			// thrown away some points that we needed at the new transform
			if (!bChaos_Collision_EnableLargeMeshManifolds)
			{
				Flags.bEnableManifoldUpdate = Flags.bEnableManifoldUpdate && !bIsMesh0 && !bIsMesh1;
			}

			// Mark probe flag now so we know which GenerateCollisions to use
			// @todo(chaos): it looks like this can be changed by a collision modifier so we should not be caching it
			Flags.bIsProbe = Shape0->GetIsProbe() || Shape1->GetIsProbe();

			return Flags;
		}

	}
}