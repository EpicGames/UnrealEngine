// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"

namespace Chaos
{
	class FPBDCollisionConstraint;

	/**
	 * @brief Return value in collision visitors to indicate whether we should continue visiting for stop
	*/
	enum class ECollisionVisitorResult
	{
		Stop,
		Continue
	};

	/*
	* @brief Flags to control which constraints will be visited in the various constraint visitor functions
	*/
	enum class ECollisionVisitorFlags : uint8
	{
		VisitActiveAwake	= 0,		// Collisions that we detected this tick and have not been disabled or put to sleep
		VisitSleeping		= 1 << 1,	// Collisions on body pairs that are sleeping
		VisitDisabled		= 1 << 2,	// Collisions that have been disabled by the user, or were not enabled this tick
		VisitExpired		= 1 << 3,	// Collisions that are still in memory for possible later reuse but not "current" (activated this frame)

		VisitDefault					= VisitActiveAwake | VisitSleeping,
		VisitAllCurrent					= VisitActiveAwake | VisitSleeping | VisitDisabled,
		VisitAllCurrentAndExpired		= VisitActiveAwake | VisitSleeping | VisitDisabled | VisitExpired,
	};
	ENUM_CLASS_FLAGS(ECollisionVisitorFlags);

}