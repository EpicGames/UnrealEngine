// Copyright Epic Games, Inc.All Rights Reserved.
#pragma once

#include "Chaos/Core.h"

namespace Chaos
{
	/**
	 * @brief Used in FContactPoint to indicate whether the contact is vertex-plane, edge-edge, etc
	 * 
	 * @note the order here is the order of preference in the solver. I.e., we like to solve Plane contacts before edge contacts before vertex contacts.
	 * This is most impotant for collisions against triangle meshes (or any concave shape) where the second shape is always the triangle, and so a PlaneVertex collision 
	 * counts as a vertex collision.
	*/
	enum class EContactPointType : int8
	{
		Unknown,
		VertexPlane,
		EdgeEdge,
		PlaneVertex,
		VertexVertex,
	};


	/**
	 * The value used for Phi (contact separation) to indicate that it is unset/invalid.
	 */
	template<typename T>
	constexpr T InvalidPhi() 
	{ 
		return TNumericLimits<T>::Max();
	}

	/**
	 * Data returned by the low-level collision functions.
	 * 
	 * @note All data is invalid/uninitialized when IsSet() is false.
	 * @see FContactPoint, FContactPointf
	*/
	template<typename T>
	class TContactPoint
	{
	public:
		using FRealType = T;

		// Shape-space contact points on the two bodies
		TVec3<FRealType> ShapeContactPoints[2];

		// Shape-space contact normal on the second shape with direction that points away from shape 1
		TVec3<FRealType> ShapeContactNormal;

		// Contact separation (negative for overlap)
		FRealType Phi;

		// Face index of the shape we hit. Only valid for Heightfield and Trimesh contact points, otherwise INDEX_NONE
		int32 FaceIndex;

		// Whether this is a vertex-plane contact, edge-edge contact etc.
		EContactPointType ContactType;

		TContactPoint()
			: Phi(InvalidPhi<FRealType>())
			, FaceIndex(INDEX_NONE)
			, ContactType(EContactPointType::Unknown)
		{
		}

		template<typename U>
		TContactPoint(const TContactPoint<U>& Other)
			: ShapeContactPoints{ TVec3<FRealType>(Other.ShapeContactPoints[0]), TVec3<FRealType>(Other.ShapeContactPoints[1]) }
			, ShapeContactNormal{ TVec3<FRealType>(Other.ShapeContactNormal) }
			, Phi(FRealType(Other.Phi))
			, FaceIndex(Other.FaceIndex)
			, ContactType(Other.ContactType)
		{
		}

		// Whether the contact point has been set up with contact data
		bool IsSet() const 
		{ 
			return (Phi != InvalidPhi<FRealType>());
		}
	};

	// Explicit types of TContactPoint for float and double. We should use the float version whenever the contacts are in shape-relative
	// space, which is most of the time (some collision functions may choose to generate world-space contacts which are later converted
	// to local space, and they should therefore use the double version until the local-space conversion).
	using FContactPoint = TContactPoint<FReal>;
	using FContactPointf = TContactPoint<FRealSingle>;


	/**
	 * @brief A single point in a contact manifold.
	 * Each Collision Constraint will have up to 4 of these.
	*/
	class FManifoldPoint
	{
	public:
		union FFlags
		{
			FFlags() { Reset(); }

			void Reset() { Bits = 0; }

			struct
			{
				uint8 bDisabled : 1;						// Whether the point was disabled by edge pruning etc
				uint8 bWasRestored : 1;						// Whether the point was retored from the previous frame due to lack of movement
				uint8 bWasReplaced : 1;						// @todo(chaos): remove this
				uint8 bHasStaticFrictionAnchor : 1;			// Whether our static friction anchor was recovered from a prior tick
				uint8 bInitialContact : 1;					// Whether this is a new contact this tick for handling initial penetration
				uint8 bDisablePositionSolve : 1;			// Whether we should disable position solve on this manifold point
				uint8 bDisableVelocitySolve : 1;			// Whether we should disable velocity solve on this manifold point
				uint8 bDisableFriction : 1;					// Whether we should disable friction solve on this manifold point
			};
			uint8 Bits;
		};

		// Verify that all FFlags data can be initialized to zero (even unused bits)
		static_assert(sizeof(FFlags) == sizeof(FFlags::Bits), "FManifoldPoint::FFlags has exceeded capacity. Change to uint16 or uint32.");

		FManifoldPoint()
			: ContactPoint()
			, Flags()
			, TargetPhi(0)
			, InitialPhi(0)
			, ShapeAnchorPoints{ FVec3f(0), FVec3f(0) }
			, InitialShapeContactPoints{ FVec3f(0), FVec3f(0) }
		{}

		FManifoldPoint(const FContactPointf& InContactPoint)
			: ContactPoint(InContactPoint)
			, Flags()
			, TargetPhi(0)
			, InitialPhi(0)
			, ShapeAnchorPoints{ FVec3f(0), FVec3f(0) }
			, InitialShapeContactPoints{ FVec3f(0), FVec3f(0) }
		{}

		FContactPointf ContactPoint;			// Contact point results of low-level collision detection
		FFlags Flags;							// Various flags
		FRealSingle TargetPhi;					// Usually 0, but can be used to add padding or penetration (e.g., via a collision modifer)
		FRealSingle InitialPhi;					// Non-resolved initial penetration
		FVec3f ShapeAnchorPoints[2];			// When static friction holds, the contact points on each shape when static friction contact was made
		FVec3f InitialShapeContactPoints[2];	// ShapeContactPoints when the constraint was first initialized. Used to track reusablility
	};

	/**
	 * World-space contact point data
	 */
	class FWorldContactPoint
	{
	public:
		// World-space contact point relative to each particle's center of mass
		FVec3f RelativeContactPoints[2];

		// World-space contact normal and tangents
		FVec3f ContactNormal;
		FVec3f ContactTangentU;
		FVec3f ContactTangentV;

		// Errors to correct along each of the contact axes
		FRealSingle ContactDeltaNormal;
		FRealSingle ContactDeltaTangentU;
		FRealSingle ContactDeltaTangentV;

		// Target velocity along the normal direction
		FRealSingle ContactTargetVelocityNormal;
	};

	/**
	 * @brief The friction data for a manifold point
	 * This is the information that needs to be stored between ticks to implement static friction.
	*/
	class FSavedManifoldPoint
	{
	public:
		explicit FSavedManifoldPoint(const FVec3f& InShapeContactPoint0, const FVec3f& InShapeContactPoint1, FRealSingle InInitialPhi)
			: ShapeContactPoints{ InShapeContactPoint0, InShapeContactPoint1 }
			, InitialPhi(InInitialPhi)
		{
		}

		FVec3f ShapeContactPoints[2];			// Contact anchor points for friction
		FRealSingle InitialPhi;					// Non-resolved initial penetration
	};

	class FManifoldPointResult
	{
	public:
		FManifoldPointResult()
		{
			Reset();
		}

		void Reset()
		{
			NetPushOut = FVec3f(0);
			NetImpulse = FVec3f(0);
			bIsValid = false;
			bInsideStaticFrictionCone = false;
		}

		FVec3f NetPushOut;						// Total pushout applied at this contact point
		FVec3f NetImpulse;						// Total impulse applied by this contact point
		uint8 bIsValid : 1;
		uint8 bInsideStaticFrictionCone : 1;
	};

}
