// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/CollisionResolutionTypes.h"
#include "Chaos/ImplicitFwd.h"
#include "Chaos/TriangleMeshImplicitObject.h"

namespace Chaos
{
	class FHeightField;
	
	/**
	 * @brief Given a sweep result, calculate the sweep time at which the penetration depth will be TargetPenetration
	 * This is based on the initial and final contact separation from the sweep test but modified so that
	 * - we ignore separating contacts (increasing Phi)
	 * - we ignore contacts that are separated at T=1 (EndPhi > 0)
	 * - we ignore contacts if the penetration is less than the IgnorePenetration
	 * - we calculate the TOI that leaves a penetration depth of TargetPenetration (except when initially overlapping by more than TargetPenetration, in which case TOI=0)
	 * 
	 * @param SweepLength The length of the sweep test
	 * @param HitDistance The distance along the sweep where the shapes first touch
	 * @param IgnorePenetration Ignore contacts if the depth at T=1 is less than this
	 * @param TargetPenetration Calculate the TOI for this penetration depth
	 * @param OutTOI The sweep time at which we hit the TargetPenetration, or "infinity" (numeric max)
	 * @param OutPhi The depth at OutTOI, which is usually TargetPenetration unless OutTOI is zero
	 * @return true if we have a TOI less than 1
	*/
	bool ComputeSweptContactTOIAndPhiAtTargetPenetration(const FReal DirDotNormal, const FReal SweepLength, const FReal HitDistance, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& OutTOI, FReal& OutPhi);
	bool ComputeSweptContactTOIAndPhiAtTargetPenetration(const FVec3& ContactNormal, const FVec3& Dir, const FReal SweepLength, const FReal HitDistance, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& OutTOI, FReal& OutPhi);
	void LegacyComputeSweptContactTOIAndPhiAtTargetPenetration(const FReal DirDotNormal, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& InOutTOI, FReal& InOutPhi);


	template <typename GeometryB>
	FContactPoint GJKImplicitSweptContactPoint(const FImplicitObject& A, const FRigidTransform3& AStartTransform, const GeometryB& B, const FRigidTransform3& BTransform, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& TOI);

	
	template <typename GeometryA, typename GeometryB>
	FContactPoint GJKImplicitContactPoint(const FImplicitObject& A, const FRigidTransform3& ATransform, const GeometryB& B, const FRigidTransform3& BTransform, const FReal CullDistance);

	FContactPoint SphereSphereContactPoint(const FSphere& Sphere1, const FRigidTransform3& Sphere1Transform, const FSphere& Sphere2, const FRigidTransform3& Sphere2Transform, const FRealSingle CullDistance);
	FContactPoint SpherePlaneContactPoint(const FSphere& Sphere, const FRigidTransform3& SphereTransform, const TPlane<FReal, 3>& Plane, const FRigidTransform3& PlaneTransform);
	FContactPoint SphereBoxContactPoint(const FSphere& Sphere, const FRigidTransform3& SphereTransform, const FImplicitBox3& Box, const FRigidTransform3& BoxTransform);
	FContactPoint SphereCapsuleContactPoint(const FSphere& A, const FRigidTransform3& ATransform, const FCapsule& B, const FRigidTransform3& BTransform);

	template <typename TriMeshType>
	FContactPoint SphereTriangleMeshContactPoint(const FSphere& A, const FRigidTransform3& ATransform, const TriMeshType& B, const FRigidTransform3& BTransform, const FReal CullDistance);

	template<typename TriMeshType>
	FContactPoint SphereTriangleMeshSweptContactPoint(const FSphere& A, const FRigidTransform3& ATransform, const TriMeshType& B, const FRigidTransform3& BStartTransform, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& TOI);

	FContactPoint BoxHeightFieldContactPoint(const FImplicitBox3& A, const FRigidTransform3& ATransform, const FHeightField& B, const FRigidTransform3& BTransform, const FReal CullDistance);

	template <typename TriMeshType>
	FContactPoint BoxTriangleMeshContactPoint(const FImplicitBox3& A, const FRigidTransform3& ATransform, const TriMeshType& B, const FRigidTransform3& BTransform, const FReal CullDistance);

	FContactPoint SphereHeightFieldContactPoint(const FSphere& A, const FRigidTransform3& ATransform, const FHeightField& B, const FRigidTransform3& BTransform, const FReal CullDistance);
	FContactPoint CapsuleHeightFieldContactPoint(const FCapsule& A, const FRigidTransform3& ATransform, const FHeightField& B, const FRigidTransform3& BTransform, const FReal CullDistance);
	template <typename TriMeshType>
	FContactPoint CapsuleTriangleMeshContactPoint(const FCapsule& A, const FRigidTransform3& ATransform, const TriMeshType& B, const FRigidTransform3& BTransform, const FReal CullDistance);
	template <typename TriMeshType>
	FContactPoint CapsuleTriangleMeshSweptContactPoint(const FCapsule& A, const FRigidTransform3& ATransform, const TriMeshType& B, const FRigidTransform3& BStartTransform, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& TOI);

	FContactPoint ConvexHeightFieldContactPoint(const FImplicitObject& A, const FRigidTransform3& ATransform, const FHeightField& B, const FRigidTransform3& BTransform, const FReal CullDistance);
	FContactPoint ConvexTriangleMeshContactPoint(const FImplicitObject& A, const FRigidTransform3& ATransform, const FImplicitObject& B, const FRigidTransform3& BTransform, const FReal CullDistance);
	template <typename TriMeshType>
	FContactPoint ConvexTriangleMeshSweptContactPoint(const FImplicitObject& A, const FRigidTransform3& ATransform, const TriMeshType& B, const FRigidTransform3& BStartTransform, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& TOI);

	FContactPoint CapsuleCapsuleContactPoint(const FCapsule& A, const FRigidTransform3& ATransform, const FCapsule& B, const FRigidTransform3& BTransform);
	FContactPoint CapsuleBoxContactPoint(const FCapsule& A, const FRigidTransform3& ATransform, const FImplicitBox3& B, const FRigidTransform3& BTransform, const FVec3& InitialDir);
	
	
}
