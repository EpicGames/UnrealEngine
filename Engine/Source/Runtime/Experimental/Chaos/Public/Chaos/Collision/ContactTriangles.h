// Copyright Epic Games, Inc.All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Templates/UnrealTemplate.h"

#include "Chaos/Core.h"
#include "Chaos/Triangle.h"

namespace Chaos
{
	// Given a point and a triangle, check if the point is on an edge or vertex and return vertex indices if so.
	// If Position is at one of the vertices, OutEdgeVertexIndexA will an index into Vertices
	// If Position is one one of the edges, both OutEdgeVertexIndexA and OutEdgeVertexIndexB will be an index into Vertices
	// If both OutEdgeVertexIndexA and OutEdgeVertexIndexB are INDEX_NONE, Position is not on the triangle edges
	inline bool GetTriangleEdgeVerticesAtPosition(
		const FVec3& Position, 
		const FVec3 VertexA, const FVec3 VertexB, const FVec3 VertexC,
		int32& OutEdgeVertexIndexA, int32& OutEdgeVertexIndexB, 
		const FReal BaryCentricTolerance = UE_KINDA_SMALL_NUMBER)
	{
		OutEdgeVertexIndexA = INDEX_NONE;
		OutEdgeVertexIndexB = INDEX_NONE;

		const FVec3 BaryCentric = ToBarycentric(Position, VertexA, VertexB, VertexC);

		// Is it a vertex contact?
		if (FMath::IsNearlyEqual(BaryCentric.X, FReal(1), BaryCentricTolerance))
		{
			OutEdgeVertexIndexA = 0;
			return true;
		}
		if (FMath::IsNearlyEqual(BaryCentric.Y, FReal(1), BaryCentricTolerance))
		{
			OutEdgeVertexIndexA = 1;
			return true;
		}
		if (FMath::IsNearlyEqual(BaryCentric.Z, FReal(1), BaryCentricTolerance))
		{
			OutEdgeVertexIndexA = 2;
			return true;
		}

		// Is it an edge contact?
		if (FMath::IsNearlyEqual(BaryCentric.X, FReal(0), BaryCentricTolerance))
		{
			OutEdgeVertexIndexA = 1;
			OutEdgeVertexIndexB = 2;
			return true;
		}
		if (FMath::IsNearlyEqual(BaryCentric.Y, FReal(0), BaryCentricTolerance))
		{
			OutEdgeVertexIndexA = 2;
			OutEdgeVertexIndexB = 0;
			return true;
		}
		if (FMath::IsNearlyEqual(BaryCentric.Z, FReal(0), BaryCentricTolerance))
		{
			OutEdgeVertexIndexA = 0;
			OutEdgeVertexIndexB = 1;
			return true;
		}

		return false;
	}

	inline bool GetTriangleEdgeVerticesAtPosition(
		const FVec3& Position,
		const FVec3 Vertices[],
		int32& OutEdgeVertexIndexA, int32& OutEdgeVertexIndexB,
		const FReal BaryCentricTolerance = UE_KINDA_SMALL_NUMBER)
	{
		return GetTriangleEdgeVerticesAtPosition(
			Position, 
			Vertices[0], Vertices[1], Vertices[2], 
			OutEdgeVertexIndexA, OutEdgeVertexIndexB, 
			BaryCentricTolerance);
	}


	/**
	 * @brief Data held alongside contact points when generating contacts against a (likely non-convex) mesh of triangles
	*/
	class FContactTriangle
	{
	public:
		// Triangle data
		FVec3 Vertices[3];
		FVec3 FaceNormal;

		// Indices into the mesh (trimesh or heightfield) that generated the triangle
		int32 VertexIndices[3];
		int32 TriangleIndex;

		// Does this triangle contains the specified vertex? (VertexIndex is an index into the owning mesh's vertices)
		inline bool HasVertexIndex(const int32 VertexIndex) const
		{
			return (VertexIndices[0] == VertexIndex) || (VertexIndices[1] == VertexIndex) || (VertexIndices[2] == VertexIndex);
		}

		// Get the centroid of the triangle
		inline FVec3 GetCentroid() const
		{
			return (Vertices[0] + Vertices[1] + Vertices[2]) / FReal(3);
		}

		// Get the positions of the other two vertices in the triangle. (VertexIndex is an index into the owning mesh's vertices)
		inline bool GetOtherVertexPositions(const int32 VertexIndex, FVec3& OutVertex0, FVec3& OutVertex1) const
		{
			if (VertexIndex == VertexIndices[0])
			{
				OutVertex0 = Vertices[1];
				OutVertex1 = Vertices[2];
				return true;
			}
			else if (VertexIndex == VertexIndices[1])
			{
				OutVertex0 = Vertices[2];
				OutVertex1 = Vertices[0];
				return true;
			}
			else if (VertexIndex == VertexIndices[2])
			{
				OutVertex0 = Vertices[0];
				OutVertex1 = Vertices[1];
				return true;
			}
			return false;
		}

		// Given a contact point on a triangle, check if it is an edge contact or vertex contact and return vertex indices if so
		// @todo(chaos): this function not should be unnecessary. Collision detection should return the vertex indices for the vertex/edge/face that we collide with
		inline bool GetEdgeVerticesAtPosition(const FVec3& ContactPosition, FVec3& OutEdgeVertexA, FVec3& OutEdgeVertexB, int32& OutEdgeVertexIndexA, int32& OutEdgeVertexIndexB, const FReal BaryCentricTolerance = UE_KINDA_SMALL_NUMBER) const
		{
			OutEdgeVertexIndexA = INDEX_NONE;
			OutEdgeVertexIndexB = INDEX_NONE;

			int32 TriVertexIndexA, TriVertexIndexB;
			if (GetTriangleEdgeVerticesAtPosition(ContactPosition, Vertices, TriVertexIndexA, TriVertexIndexB, BaryCentricTolerance))
			{
				if (TriVertexIndexA != INDEX_NONE)
				{
					OutEdgeVertexIndexA = VertexIndices[TriVertexIndexA];
					OutEdgeVertexA = Vertices[TriVertexIndexA];
				}
				if (TriVertexIndexB != INDEX_NONE)
				{
					OutEdgeVertexIndexB = VertexIndices[TriVertexIndexB];
					OutEdgeVertexB = Vertices[TriVertexIndexB];
				}
				return true;
			}

			return false;
		}
	};

	// ID for a vertex in a trangle mesh, used by FTriangleContactPointData
	using FContactVertexID = int32;

	// ID for an edge in a trangle mesh, used by FTriangleContactPointData
	struct FContactEdgeID
	{
		FORCEINLINE FContactEdgeID()
			: VertexIDs{ INDEX_NONE, INDEX_NONE }
		{
		}

		FORCEINLINE FContactEdgeID(const FContactVertexID VertexIndexA, const FContactVertexID VertexIndexB)
		{
			// EdgeID is the same if we swap the vertex indices
			// The check for INDEX_NONE allows us to use an EdgeID as a VertexID without swapping order
			if ((VertexIndexA < VertexIndexB) || (VertexIndexB == INDEX_NONE))
			{
				VertexIDs[0] = VertexIndexA;
				VertexIDs[1] = VertexIndexB;
			}
			else
			{
				VertexIDs[0] = VertexIndexB;
				VertexIDs[1] = VertexIndexA;
			}
		}

		FORCEINLINE bool IsValid() const
		{
			return (VertexIDs[0] != INDEX_NONE) && (VertexIDs[1] != INDEX_NONE);
		}

		FORCEINLINE friend bool operator==(const FContactEdgeID& L, const FContactEdgeID& R)
		{
			return L.EdgeID == R.EdgeID;
		}

		FORCEINLINE friend bool operator!=(const FContactEdgeID& L, const FContactEdgeID& R)
		{
			return L.EdgeID != R.EdgeID;
		}

		FORCEINLINE friend bool operator<(const FContactEdgeID& L, const FContactEdgeID& R)
		{
			return L.EdgeID < R.EdgeID;
		}

		FORCEINLINE friend uint32 GetTypeHash(const FContactEdgeID& V)
		{
			return ::GetTypeHash(V.EdgeID);
		}

		union
		{
			FContactVertexID VertexIDs[2];
			uint64 EdgeID;
		};
	};

	
	//class UE_DEPRECATED(5.4, "No longer used") FTriangleContactPoint : public FContactPoint
	//{
	//public:
	//	FTriangleContactPoint()
	//		: FContactPoint()
	//		, EdgeID(INDEX_NONE, INDEX_NONE)
	//		, VertexID(INDEX_NONE)
	//		, ContactTriangleIndex(INDEX_NONE)
	//	{
	//	}

	//	FTriangleContactPoint(const FContactPoint& InContactPoint)
	//		: FContactPoint(InContactPoint)
	//		, EdgeID(INDEX_NONE, INDEX_NONE)
	//		, VertexID(INDEX_NONE)
	//		, ContactTriangleIndex(INDEX_NONE)
	//	{
	//	}

	//	FContactEdgeID EdgeID;
	//	FContactVertexID VertexID;

	//	int32 ContactTriangleIndex;
	//};

	/**
	* An ID for an Edge or Vertex in a triangle mesh.
	*/
	struct FContactEdgeOrVertexID
	{
	public:
		FORCEINLINE FContactEdgeOrVertexID()
			: EdgeID()
		{
		}

		FORCEINLINE FContactEdgeOrVertexID(const FContactVertexID VertexIndex)
		{
			EdgeID.VertexIDs[0] = VertexIndex;
			EdgeID.VertexIDs[1] = INDEX_NONE;
		}

		FORCEINLINE FContactEdgeOrVertexID(const FContactVertexID VertexIndexA, const FContactVertexID VertexIndexB)
		{
			EdgeID = FContactEdgeID(VertexIndexA, VertexIndexB);
		}

		FORCEINLINE bool IsValid() const
		{
			return (EdgeID.VertexIDs[0] != INDEX_NONE);
		}

		FORCEINLINE bool IsVertex() const
		{
			return (EdgeID.VertexIDs[0] != INDEX_NONE) && (EdgeID.VertexIDs[1] == INDEX_NONE);
		}

		FORCEINLINE bool IsEdge() const
		{
			return (EdgeID.VertexIDs[0] != INDEX_NONE) && (EdgeID.VertexIDs[1] != INDEX_NONE);
		}

		FORCEINLINE const FContactVertexID& GetVertexID() const
		{
			return EdgeID.VertexIDs[0];
		}

		FORCEINLINE const FContactEdgeID& GetEdgeID() const
		{
			return EdgeID;
		}

		FORCEINLINE friend bool operator==(const FContactEdgeOrVertexID& L, const FContactEdgeOrVertexID& R)
		{
			return L.EdgeID == R.EdgeID;
		}

		FORCEINLINE friend bool operator!=(const FContactEdgeOrVertexID& L, const FContactEdgeOrVertexID& R)
		{
			return L.EdgeID != R.EdgeID;
		}

		FORCEINLINE friend bool operator<(const FContactEdgeOrVertexID& L, const FContactEdgeOrVertexID& R)
		{
			return L.EdgeID < R.EdgeID;
		}

		FORCEINLINE friend uint32 GetTypeHash(const FContactEdgeOrVertexID& V)
		{
			return (V.IsVertex()) ? V.GetVertexID() : GetTypeHash(V.GetEdgeID());
		}

	private:
		// We treat the EdgeID as a VertexID when only one vertex ID is set
		FContactEdgeID EdgeID;
	};

	/**
	 * @brief Extended data used when processing contactpoints on a triangle mesh.
	 * Adds information about which triangle we hit (in the local array of FoOntactTriangles)
	 * as well as which mesh edge or vertex we hit if appropriate (used for pruning).
	*/
	class FTriangleContactPointData
	{
	public:
		FORCEINLINE FTriangleContactPointData()
			: EdgeOrVertexID()
			, ContactTriangleIndex(INDEX_NONE)
			, ContactNormalDotTriangleNormal(0)
			, bIsEnabled(false)
		{
		}

		FORCEINLINE void SetVertexID(const FContactVertexID InVertexID)
		{
			EdgeOrVertexID = FContactEdgeOrVertexID(InVertexID);
		}

		FORCEINLINE void SetEdgeID(const FContactVertexID InVertexIDA, const FContactVertexID InVertexIDB)
		{
			EdgeOrVertexID = FContactEdgeOrVertexID(InVertexIDA, InVertexIDB);
		}

		FORCEINLINE void SetEdgeOrVertexID(const FContactEdgeOrVertexID InEdgeOrVertexID)
		{
			EdgeOrVertexID = InEdgeOrVertexID;
		}

		FORCEINLINE void SetTriangleIndex(const int32 InTriangleIndex)
		{
			ContactTriangleIndex = InTriangleIndex;
		}

		FORCEINLINE void SetContactNormalDotTriangleNormal(const FRealSingle InContactNormalDotTriangleNormal)
		{
			ContactNormalDotTriangleNormal = InContactNormalDotTriangleNormal;
		}

		FORCEINLINE void SetEnabled()
		{
			bIsEnabled = true;
		}

		FORCEINLINE void SetDisabled()
		{
			bIsEnabled = false;
		}

		FORCEINLINE bool IsEnabled() const
		{
			return bIsEnabled;
		}

		FORCEINLINE int32 GetTriangleIndex() const
		{
			return ContactTriangleIndex;
		}

		FORCEINLINE bool IsVertex() const
		{
			return EdgeOrVertexID.IsVertex();
		}

		FORCEINLINE bool IsEdge() const
		{
			return EdgeOrVertexID.IsEdge();
		}

		FORCEINLINE FContactVertexID GetVertexID() const
		{
			return EdgeOrVertexID.GetVertexID();
		}

		FORCEINLINE FContactEdgeID GetEdgeID() const
		{
			return EdgeOrVertexID.GetEdgeID();
		}

		FORCEINLINE FRealSingle GetContactNormalDotTriangleNormal() const
		{
			return ContactNormalDotTriangleNormal;
		}

	private:
		FContactEdgeOrVertexID EdgeOrVertexID;
		int32 ContactTriangleIndex;
		FRealSingle ContactNormalDotTriangleNormal;
		bool bIsEnabled;
	};

	/**
	 * Holds the contacts produced when colliding a convex shape against a non-convex triangular mesh, and provides
	 * methods to reduce the total set of contact points down to a minimum manifold.
	*/
	class FContactTriangleCollector
	{
	public:
		UE_DEPRECATED(5.3, "Use the constructor below which take PhiTolerance etc")
		FContactTriangleCollector(const bool bInOneSided, const FRigidTransform3& InConvexTransform)
			: PhiTolerance(0.1)
			, DistanceTolerance(0.1)
			, NumDisabledTriangleContactPoints(0)
			, bOneSidedCollision(bInOneSided)
			, ConvexTransform(InConvexTransform)
		{
		}

		FContactTriangleCollector(
			const bool bInOneSided,
			const FReal InPhiTolerance,
			const FReal InDistanceTolerance,
			const FRigidTransform3& InConvexTransform)
			: PhiTolerance(InPhiTolerance)
			, DistanceTolerance(InDistanceTolerance)
			, NumDisabledTriangleContactPoints(0)
			, bOneSidedCollision(bInOneSided)
			, ConvexTransform(InConvexTransform)
		{
		}

		inline TArrayView<const FContactPoint> GetContactPoints() const
		{ 
			return MakeArrayView(TriangleContactPoints);
		}
		
		// Add contacts between the convex and a single triangle for later processing
		inline void AddTriangleContacts(const TArrayView<const FContactPoint>& InContactPoints, const FTriangle& Triangle, const int32 TriangleIndex, const int32 VertexIndex0, const int32 VertexIndex1, const int32 VertexIndex2, const FReal CullDistance)
		{
			if (InContactPoints.Num() > 0)
			{
				const int32 ContactTriangleIndex = AddTriangle(Triangle, TriangleIndex, VertexIndex0, VertexIndex1, VertexIndex2);
				
				const int32 NewNumContactPoints = TriangleContactPoints.Num() + InContactPoints.Num();
				TriangleContactPoints.Reserve(NewNumContactPoints);
				TriangleContactPointDatas.Reserve(NewNumContactPoints);
				
				for (const FContactPoint& ContactPoint : InContactPoints)
				{
					if (ContactPoint.Phi < CullDistance)
					{
						const FReal ContactNormalDotTriangleNormal = FVec3::DotProduct(ContactPoint.ShapeContactNormal, ContactTriangles[ContactTriangleIndex].FaceNormal);

						const int32 ContactIndex = TriangleContactPoints.Add(ContactPoint);
						TriangleContactPointDatas.AddDefaulted();

						TriangleContactPoints[ContactIndex].FaceIndex = TriangleIndex;
						TriangleContactPointDatas[ContactIndex].SetTriangleIndex(ContactTriangleIndex);
						TriangleContactPointDatas[ContactIndex].SetContactNormalDotTriangleNormal(FRealSingle(ContactNormalDotTriangleNormal));
						TriangleContactPointDatas[ContactIndex].SetEnabled();
					}
				}
			}
		}

		// To be called when all contacts have been added. Reduces the set of contacts to a minimum and transforms data into shape space
		void ProcessContacts(const FRigidTransform3& MeshToConvexTransform);

	private:
		void SetNumContacts(const int32 Num);
		void DisableContact(const int32 ContactIndex);
		void RemoveDisabledContacts();

		void BuildContactFeatureSets();
		void SortContactPointsForPruning();
		void SortContactPointsForSolving();
		void PruneEdgeAndVertexContactPoints(const bool bPruneEdges, const bool bPruneVertices);
		void FixInvalidNormalContactPoints();
		void PruneInfacingContactPoints();
		void PruneUnnecessaryContactPoints();
		void ReduceManifoldContactPointsTriangeMesh();
		void FinalizeContacts(const FRigidTransform3& MeshToConvexTransform);

		int32 AddTriangle(const FTriangle& Triangle, const int32 TriangleIndex, const int32 VertexIndex0, const int32 VertexIndex1, const int32 VertexIndex2);
		int32 GetOtherTriangleIndexSharingEdge(const int32 KnownTriangleIndex, const int32 EdgeVertexIndexA, const int32 EdgeVertexIndexB) const;
		int32 GetNextTriangleIndexSharingVertex(const int32 LastContactTriangleIndex, const int32 VertexIndex) const;

		void DebugDrawContactPoints(const FColor& Color, const FReal LineThickness);

		// The collected set of contacts with additional triangle information
		TArray<FContactPoint> TriangleContactPoints;
		TArray<FTriangleContactPointData> TriangleContactPointDatas;

		// All the triangles referenced by the TriangleContactPoints
		TArray<FContactTriangle> ContactTriangles;

		// A list of all the edges of contacts with faces. Used to prune edge contacts
		TSet<FContactEdgeID> ContactEdges;

		// A list of all the vertices of contacts with faces and edges. Used to prune vertex contacts
		TSet<FContactVertexID> ContactVertices;

		// We remove contacts that are shallower by this much compared to the deepest contact
		FReal PhiTolerance;

		// We remove contacts that are closer than this to any other contact (negative to disable this functionality)
		FReal DistanceTolerance;

		// How many disabled contacts are in the TriangleContactPoints and TriangleContactPointDatas arrays (until the next Pack operation)
		int32 NumDisabledTriangleContactPoints;

		// Whether we want single-sided collision and therefore reject all normal opposing the triangle faces
		bool bOneSidedCollision;

		// Local space for the triangles
		// @todo(chaos): rename this
		FRigidTransform3 ConvexTransform;
	};

}
