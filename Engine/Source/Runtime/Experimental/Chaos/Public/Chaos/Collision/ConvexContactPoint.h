// Copyright Epic Games, Inc.All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/Collision/ContactPoint.h"
#include "Chaos/Collision/ConvexFeature.h"

namespace Chaos::Private
{
	// A contact point between two convex features
	template<typename T>
	class TConvexContactPoint
	{
	public:
		using FRealType = T;

		TConvexContactPoint()
		{
			Reset();
		}

		inline void Reset()
		{
			Phi = InvalidPhi<FRealType>();
		}

		inline bool IsSet() const
		{
			return Phi != InvalidPhi<FRealType>();
		}

		// Convert a feature pair into a contact type (used by callbacks, events, etc)
		inline EContactPointType GetContactPointType() const
		{
			if (!IsSet())
			{
				return EContactPointType::Unknown;
			}

			// Both features should be set to something
			if ((Features[0].FeatureType == EConvexFeatureType::Unknown) || (Features[1].FeatureType == EConvexFeatureType::Unknown))
			{
				return EContactPointType::Unknown;
			}

			// Plane-Plane, Edge-Plane and Vertex-Plane are treated as Vertex-Plane
			if (Features[1].FeatureType == EConvexFeatureType::Plane)
			{
				return EContactPointType::VertexPlane;
			}

			// Plane-Vertex and Plane-Edge are treated as Plane-Vertex
			if (Features[0].FeatureType == EConvexFeatureType::Plane)
			{
				return EContactPointType::PlaneVertex;
			}

			// Vertex-Vertex, Edge-Vertex and Edge-Edge are treated as Edge-Edge
			return EContactPointType::EdgeEdge;
		}

		FConvexFeature Features[2];
		TVec3<FRealType> ShapeContactPoints[2];
		TVec3<FRealType> ShapeContactNormal;
		FRealType Phi;
	};

	using FConvexContactPoint = TConvexContactPoint<FReal>;
	using FConvexContactPointf = TConvexContactPoint<FRealSingle>;
}