// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/Capsule.h"
#include "Chaos/Collision/ContactPoint.h"
#include "Chaos/Collision/ContactTriangles.h"
#include "Chaos/Collision/ConvexFeature.h"
#include "Chaos/Triangle.h"

namespace Chaos::Private
{
	// Project a convex onto an axis and return the projected range as well as the vertex indices that bound the range
	template <typename ConvexType>
	inline void ProjectOntoAxis(const ConvexType& Convex, const FVec3& AxisN, const FVec3& AxisX, FReal& PMin, FReal& PMax, int32& MinVertexIndex, int32& MaxVertexIndex, TArrayView<FReal>* VertexDs)
	{
		PMin = std::numeric_limits<FReal>::max();
		PMax = std::numeric_limits<FReal>::lowest();
		for (int32 VertexIndex = 0; VertexIndex < Convex.NumVertices(); ++VertexIndex)
		{
			const FVec3 V = Convex.GetVertex(VertexIndex);
			const FReal D = FVec3::DotProduct(V - AxisX, AxisN);
			if (D < PMin)
			{
				PMin = D;
				MinVertexIndex = VertexIndex;
			}
			if (D > PMax)
			{
				PMax = D;
				MaxVertexIndex = VertexIndex;
			}
			if (VertexDs != nullptr)
			{
				(*VertexDs)[VertexIndex] = D;
			}
		}
	}

	// Project a triangle onto an axis and return the projected range as well as the vertex indices that bound the range
	inline void ProjectOntoAxis(const FTriangle& Triangle, const FVec3& AxisN, const FVec3& AxisX, FReal& PMin, FReal& PMax, int32& MinVertexIndex, int32& MaxVertexIndex)
	{
		const FVec3& V0 = Triangle.GetVertex(0);
		const FVec3& V1 = Triangle.GetVertex(1);
		const FVec3& V2 = Triangle.GetVertex(2);
		const FReal Ds[3] =
		{
			FVec3::DotProduct(V0 - AxisX, AxisN),
			FVec3::DotProduct(V1 - AxisX, AxisN),
			FVec3::DotProduct(V2 - AxisX, AxisN)
		};
		MinVertexIndex = FMath::Min3Index(Ds[0], Ds[1], Ds[2]);
		MaxVertexIndex = FMath::Max3Index(Ds[0], Ds[1], Ds[2]);
		PMin = Ds[MinVertexIndex];
		PMax = Ds[MaxVertexIndex];
	}

	// Project a capsule segment onto an axis and return the projected range as well as the vertex indices that bound the range
	inline void ProjectOntoAxis(const FCapsule& Capsule, const FVec3& AxisN, const FVec3& AxisX, FReal& PMin, FReal& PMax, int32& MinVertexIndex, int32& MaxVertexIndex)
	{
		const FVec3 V0 = Capsule.GetX1f();
		const FVec3 V1 = Capsule.GetX2f();
		const FReal D0 = FVec3::DotProduct(V0 - AxisX, AxisN);
		const FReal D1 = FVec3::DotProduct(V1 - AxisX, AxisN);
		if (D0 < D1)
		{
			MinVertexIndex = 0;
			MaxVertexIndex = 1;
			PMin = D0;
			PMax = D1;
		}
		else
		{
			MinVertexIndex = 1;
			MaxVertexIndex = 0;
			PMin = D1;
			PMax = D0;
		}
	}

	// Get the convex feature at the specific position and normal
	template<typename ConvexType>
	bool GetConvexFeature(const ConvexType& Convex, const FVec3& Position, const FVec3& Normal, Private::FConvexFeature& OutFeature)
	{
		const FReal NormalTolerance = FReal(1.e-6);
		const FReal PositionTolerance = FReal(1.e-4);
		const FReal ToleranceSizeMultiplier = Convex.BoundingBox().Extents().GetAbsMax();
		const FReal EdgeNormalTolerance = ToleranceSizeMultiplier * FReal(1.e-3);

		int32 BestPlaneIndex = INDEX_NONE;
		FReal BestPlaneDotNormal = FReal(-1);

		// Get the support vertex along the normal (which must point away from the convex)
		int SupportVertexIndex = INDEX_NONE;
		Convex.SupportCore(Normal, 0, nullptr, SupportVertexIndex);

		if (SupportVertexIndex != INDEX_NONE)
		{
			// See if the normal matches a face normal for any face using the vertex
			int32 VertexPlanes[16];
			int32 NumVertexPlanes = Convex.FindVertexPlanes(SupportVertexIndex, VertexPlanes, UE_ARRAY_COUNT(VertexPlanes));
			for (int32 VertexPlaneIndex = 0; VertexPlaneIndex < NumVertexPlanes; ++VertexPlaneIndex)
			{
				const int32 PlaneIndex = VertexPlanes[VertexPlaneIndex];
				FVec3 PlaneN, PlaneX;
				Convex.GetPlaneNX(PlaneIndex, PlaneN, PlaneX);
				const FReal PlaneDotNormal = FVec3::DotProduct(PlaneN, Normal);
				if (FMath::IsNearlyEqual(PlaneDotNormal, FReal(1), NormalTolerance))
				{
					OutFeature.FeatureType = Private::EConvexFeatureType::Plane;
					OutFeature.PlaneIndex = PlaneIndex;
					OutFeature.PlaneFeatureIndex = 0;
					return true;
				}

				if (PlaneDotNormal > BestPlaneDotNormal)
				{
					BestPlaneIndex = PlaneIndex;
					BestPlaneDotNormal = PlaneDotNormal;
				}
			}

			// See if any of the edges using the vertex are perpendicular to the normal
			// @todo(chaos): we could visit the vertex edges here rather than use the plane edges
			if (BestPlaneIndex != INDEX_NONE)
			{
				int32 BestPlaneVertexIndex = INDEX_NONE;

				const int32 NumPlaneVertices = Convex.NumPlaneVertices(BestPlaneIndex);
				for (int32 PlaneVertexIndex = 0; PlaneVertexIndex < NumPlaneVertices; ++PlaneVertexIndex)
				{
					const int32 VertexIndex0 = Convex.GetPlaneVertex(BestPlaneIndex, PlaneVertexIndex);
					const int32 VertexIndex1 = (PlaneVertexIndex == NumPlaneVertices - 1) ? Convex.GetPlaneVertex(BestPlaneIndex, 0) : Convex.GetPlaneVertex(BestPlaneIndex, PlaneVertexIndex + 1);

					if (VertexIndex0 == SupportVertexIndex)
					{
						BestPlaneVertexIndex = PlaneVertexIndex;
					}

					if ((VertexIndex0 == SupportVertexIndex) || (VertexIndex1 == SupportVertexIndex))
					{
						const FVec3 Vertex0 = Convex.GetVertex(VertexIndex0);
						const FVec3 Vertex1 = Convex.GetVertex(VertexIndex1);
						const FVec3 EdgeDelta = Vertex1 - Vertex0;
						const FReal EdgeDotNormal = FVec3::DotProduct(EdgeDelta, Normal);
						if (FMath::Abs(EdgeDotNormal) < EdgeNormalTolerance)
						{
							// @todo(chaos): we need to be able to get an EdgeIndex (probably half edge index)
							// Also, we probably want both the plane index and the edge index
							OutFeature.FeatureType = Private::EConvexFeatureType::Edge;
							OutFeature.PlaneIndex = BestPlaneIndex;
							OutFeature.PlaneFeatureIndex = PlaneVertexIndex;
							return true;
						}
					}
				}

				// Not a face or edge, so it should be the SupportVertex, but we need to specify the 
				// plane and plane-index rather than the convex vertex index (which we found just above)
				if (BestPlaneVertexIndex != INDEX_NONE)
				{
					OutFeature.FeatureType = Private::EConvexFeatureType::Vertex;
					OutFeature.PlaneIndex = BestPlaneIndex;
					OutFeature.PlaneFeatureIndex = BestPlaneVertexIndex;
				}
				return true;
			}
		}

		return false;
	}

	// Get the triangle feature at the specific position and normal
	inline bool GetTriangleFeature(const FTriangle& Triangle, const FVec3& TriangleNormal, const FVec3& Position, const FVec3& Normal, Private::FConvexFeature& OutFeature)
	{
		// @todo(chaos): pass in the triangle normal - we almost certainly calculated it elsewhere
		// NOTE: The normal epsilon needs to be less than the maximu error that GJK/EPA produces when it hits a degenerate
		// case, which can happen when we have almost exact face-to-face contact. The max error is hard to know, since it 
		// depends on the state of GJK on the iteration before it hits its tolerance, but seems to be typically ~0.01
		const FReal NormalEpsilon = FReal(0.02);
		const FReal NormalDot = FVec3::DotProduct(Normal, TriangleNormal);
		if (FMath::IsNearlyEqual(NormalDot, FReal(1), NormalEpsilon))
		{
			OutFeature.FeatureType = Private::EConvexFeatureType::Plane;
			OutFeature.PlaneIndex = 0;
			OutFeature.PlaneFeatureIndex = 0;
			return true;
		}

		const FReal BarycentricTolerance = FReal(1.e-6);
		int32 VertexIndex0, VertexIndex1;
		if (GetTriangleEdgeVerticesAtPosition(Position, &Triangle.GetVertex(0), VertexIndex0, VertexIndex1, BarycentricTolerance))
		{
			if ((VertexIndex0 != INDEX_NONE) && (VertexIndex1 != INDEX_NONE))
			{
				OutFeature.FeatureType = Private::EConvexFeatureType::Edge;
				OutFeature.PlaneIndex = 0;
				OutFeature.PlaneFeatureIndex = VertexIndex0;
				return true;
			}
			else if (VertexIndex0 != INDEX_NONE)
			{
				OutFeature.FeatureType = Private::EConvexFeatureType::Vertex;
				OutFeature.PlaneIndex = 0;
				OutFeature.PlaneFeatureIndex = VertexIndex0;
				return true;
			}
			else if (VertexIndex1 != INDEX_NONE)
			{
				OutFeature.FeatureType = Private::EConvexFeatureType::Vertex;
				OutFeature.PlaneIndex = 0;
				OutFeature.PlaneFeatureIndex = VertexIndex1;
				return true;
			}
		}

		return false;
	}

	// Check whether the two edges of two convex shapes contribute to the Minkowski sum.
	// A and B are the face normals for the faces of the edge convex 1
	// C and D are the negated face normals for the faces of the edge convex 2
	template<typename RealType>
	inline bool IsOnMinkowskiSumConvexConvex(const TVec3<RealType>& A, const TVec3<RealType>& B, const TVec3<RealType>& C, const TVec3<RealType>& D, const RealType Tolerance = 1.e-2f)
	{
		const TVec3<RealType> BA = TVec3<RealType>::CrossProduct(B, A);
		const TVec3<RealType> DC = TVec3<RealType>::CrossProduct(D, C);
		const RealType CBA = TVec3<RealType>::DotProduct(C, BA);
		const RealType DBA = TVec3<RealType>::DotProduct(D, BA);
		const RealType ADC = TVec3<RealType>::DotProduct(A, DC);
		const RealType BDC = TVec3<RealType>::DotProduct(B, DC);

		return ((CBA * DBA) < -Tolerance) && ((ADC * BDC) < -Tolerance) && ((CBA * BDC) > Tolerance);
	}

	// Check whether the convex-triangle edge pair form part of the Minkowski Sum. Only edge pairs
	// that contribute to the Minkowki Sum surface need to be checked for separation. The inputs
	// are the convex normals for the two faces that share the convex edge, and the normal and
	// edge vector of the triangle.
	// 
	// This is a custom version of IsOnMinkowskiSumConvexConvex for triangles where the two normals 
	// are directly opposing and therefore the regular edge vector calculation returns zero.
	// 
	// @param A ConvexNormalA
	// @param B ConvexNormalB
	// @param BA ConvexEdge
	// @param C TriNormal (negated)
	// @param DC TriEdge
	inline bool IsOnMinkowskiSumConvexTriangle(const FVec3& A, const FVec3& B, const FVec3& BA, const FVec3& C, const FVec3& DC)
	{
		const FReal CBA = FVec3::DotProduct(C, BA);		// TriNormal | ConvexEdge
		const FReal ADC = FVec3::DotProduct(A, DC);		// ConvexNormalA | TriEdge
		const FReal BDC = FVec3::DotProduct(B, DC);		// ConvexNormalB | TriEdge

		const FReal Tolerance = 1.e-2f;
		return ((ADC * BDC) < -Tolerance) && ((CBA * BDC) > Tolerance);
	}

}

namespace Chaos
{
	template <typename ConvexType>
	UE_DEPRECATED(5.4, "Not part of public API")
	inline void ProjectOntoAxis(const ConvexType& Convex, const FVec3& AxisN, const FVec3& AxisX, FReal& PMin, FReal& PMax, int32& MinVertexIndex, int32& MaxVertexIndex, TArrayView<FReal>* VertexDs)
	{
		return Private::ProjectOntoAxis(Convex, AxisN, AxisX, PMin, PMax, MinVertexIndex, MaxVertexIndex, VertexDs);
	}

	UE_DEPRECATED(5.4, "Not part of public API")
	inline void ProjectOntoAxis(const FTriangle& Triangle, const FVec3& AxisN, const FVec3& AxisX, FReal& PMin, FReal& PMax, int32& MinVertexIndex, int32& MaxVertexIndex)
	{
		return Private::ProjectOntoAxis(Triangle, AxisN, AxisX, PMin, PMax, MinVertexIndex, MaxVertexIndex);
	}

	UE_DEPRECATED(5.4, "Not part of public API")
	inline void ProjectOntoAxis(const FCapsule& Capsule, const FVec3& AxisN, const FVec3& AxisX, FReal& PMin, FReal& PMax, int32& MinVertexIndex, int32& MaxVertexIndex)
	{
		return Private::ProjectOntoAxis(Capsule, AxisN, AxisX, PMin, PMax, MinVertexIndex, MaxVertexIndex);
	}
}
