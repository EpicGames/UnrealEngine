// Copyright Epic Games, Inc.All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/Collision/ContactPoint.h"

namespace Chaos::Private
{
	// The type of a convex feature
	enum class EConvexFeatureType : int8
	{
		Unknown,
		Vertex,
		Edge,
		Plane
	};

	// Information used to identify a feature on a convex shape
	class FConvexFeature
	{
	public:
		// An index used to index into some collection (e.g., triangle index in a mesh)
		int32 ObjectIndex;

		// The plane index on the convex. E.g., See FConvex::GetPlane()
		int32 PlaneIndex;

		// For Vertex or Edge features this is either:
		// - the vertex index on the plane if PlaneIndex is set (see FConvex::GetPlaneVertex()), or
		// - the absolute vertex index if PlaneIndex is INDEX_NONE (see FConvex::GetVertex())
		int32 PlaneFeatureIndex;

		// The feature type
		EConvexFeatureType FeatureType;
	};
}