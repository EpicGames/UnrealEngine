// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Chaos/Core.h"
#include "Chaos/Collision/ContactPoint.h"
#include "Chaos/CollisionResolutionTypes.h"
#include "Chaos/Framework/UncheckedArray.h"
#include "Chaos/ImplicitFwd.h"

namespace Chaos
{
	namespace Private
	{
		template<typename ConvexType>
		extern CHAOS_API bool FindClosestFeatures(const ConvexType& Convex, const FTriangle& Triangle, const FVec3& TriangleNormal, const FVec3& ConvexRelativeMovement, const FReal CullDistance, FConvexContactPoint& OutContact);

		template<> CHAOS_API bool FindClosestFeatures<FImplicitSphere3>(const FImplicitSphere3& Convex, const FTriangle& Triangle, const FVec3& TriangleNormal, const FVec3& ConvexRelativeMovement, const FReal CullDistance, FConvexContactPoint& OutContactPoint);

		template <typename ConvexType>
		extern CHAOS_API void ConvexTriangleManifoldFromContact(const ConvexType& Convex, const FTriangle& Triangle, const FVec3& TriangleNormal, const FConvexContactPoint& Contact, const FReal CullDistance, FContactPointManifold& OutManifold);
	}

	template <typename ConvexType>
	extern CHAOS_API void ConstructConvexTriangleOneShotManifold2(const ConvexType& Convex, const FTriangle& Triangle, const FReal CullDistance, FContactPointManifold& OutContactPoints);
}