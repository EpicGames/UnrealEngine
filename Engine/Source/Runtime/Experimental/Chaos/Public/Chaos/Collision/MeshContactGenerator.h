// Copyright Epic Games, Inc.All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/Collision/ContactPoint.h"
#include "Chaos/Collision/ContactTriangles.h"
#include "Chaos/CollisionResolutionTypes.h"
#include "Chaos/Framework/HashMappedArray.h"
#include "Chaos/Framework/UncheckedArray.h"
#include "Chaos/Triangle.h"

namespace Chaos::Private
{
	class FMeshContactGeneratorSettings
	{
	public:
		FMeshContactGeneratorSettings();

		// Contacts with a dot product against the face normal above this value will not be processed in FixContactNormal
		FReal FaceNormalDotThreshold;

		// Triangle edge/vertex contacts that are more than this far from a valid normal (dot product) will be rejected rather than corrected
		FReal EdgeNormalDotRejectTolerance;

		// When backface culling is enabled, the tolerance for the dot product of the contact normal against the face normal
		FReal BackFaceCullTolerance;

		// Used to determine whether a contact is on an edge or vertex
		FReal BarycentricTolerance;

		// We don't allow more (pre-filtered) contacts than this. Any extras will be lost.
		int32 MaxContactsBufferSize;

		// Size of the hash table used to store/lookup triangle data
		int32 HashSize;

		// Whether to ignore inside normals
		// @todo(chaos): the non-culled option is not well tested and probably broken
		uint32 bCullBackFaces : 1;

		// Whether to auto-correct normals
		uint32 bFixNormals : 1;

		// Whether to sort the contacts by depth
		uint32 bSortByPhi : 1;

		// Whether to sort the contacts to improve solver convergence (distance from the center of mass)
		uint32 bSortForSolverConvergence : 1;

		// Whether to use the optimized two-pass loop over triangles in GenerateMeshContacts which skips triangles
		// that have contacts on all vertices in the second pass. This is only useful when this case occurs a lot
		// which is does for large convexes against many triangles, but rarely for capsules and spheres.
		uint32 bUseTwoPassLoop : 1;
	};

	// A triangle plus some extended data and state
	// Should be a member of FMeshContactGenerator but that causes natvis issues.
	class FMeshContactGeneratorTriangle
	{
		static constexpr FReal InvalidNormalMarker = std::numeric_limits<FReal>::max();

	public:
		FMeshContactGeneratorTriangle(const FTriangle& InTriangle, const int32 InTriangleIndex, const int32 InVertexIndex0, const int32 InVertexIndex1, const int32 InVertexIndex2)
			: Triangle(InTriangle)
			, Normal(InvalidNormalMarker)
			, TriangleIndex(InTriangleIndex)
			, VertexIndices{ InVertexIndex0, InVertexIndex1, InVertexIndex2 }
			, NumFaceEdgeCollisions(0)
			, VisitIndex(INDEX_NONE)
			, bEnabled(true)
		{
		}

		// Does this triangle contains the specified vertex? (VertexIndex is an index into the owning mesh's vertices)
		inline bool HasVertexID(const int32 VertexIndex) const
		{
			return (VertexIndices[0] == VertexIndex) || (VertexIndices[1] == VertexIndex) || (VertexIndices[2] == VertexIndex);
		}

		// Get the vertex position from the vertex ID (not the triangle-local vertex index)
		inline bool GetVertexWithID(const int32 VertexID, FVec3& OutVertex) const
		{
			if (VertexID == VertexIndices[0])
			{
				OutVertex = Triangle.GetVertex(0);
				return true;
			}
			else if (VertexID == VertexIndices[1])
			{
				OutVertex = Triangle.GetVertex(1);
				return true;
			}
			else if (VertexID == VertexIndices[2])
			{
				OutVertex = Triangle.GetVertex(2);
				return true;
			}
			return false;
		}

		inline bool GetOtherVertexIDs(const int32 VertexID, int32& OutVertexID0, int32& OutVertexID1) const
		{
			if (VertexID == VertexIndices[0])
			{
				OutVertexID0 = VertexIndices[1];
				OutVertexID1 = VertexIndices[2];
				return true;
			}
			else if (VertexID == VertexIndices[1])
			{
				OutVertexID0 = VertexIndices[2];
				OutVertexID1 = VertexIndices[0];
				return true;
			}
			else if (VertexID == VertexIndices[2])
			{
				OutVertexID0 = VertexIndices[0];
				OutVertexID1 = VertexIndices[1];
				return true;
			}
			return false;
		}

		// Get the positions of the other two vertices in the triangle. (VertexIndex is an index into the owning mesh's vertices)
		inline bool GetOtherVerticesFromID(const int32 VertexID, FVec3& OutVertex0, FVec3& OutVertex1) const
		{
			if (VertexID == VertexIndices[0])
			{
				OutVertex0 = Triangle.GetVertex(1);
				OutVertex1 = Triangle.GetVertex(2);
				return true;
			}
			else if (VertexID == VertexIndices[1])
			{
				OutVertex0 = Triangle.GetVertex(2);
				OutVertex1 = Triangle.GetVertex(0);
				return true;
			}
			else if (VertexID == VertexIndices[2])
			{
				OutVertex0 = Triangle.GetVertex(0);
				OutVertex1 = Triangle.GetVertex(1);
				return true;
			}
			return false;
		}

		int32 GetLocalVertexIndexAt(const FVec3& InPos, const FReal InTolerance) const
		{
			for (int32 LocalVertexIndex = 0; LocalVertexIndex < 3; ++LocalVertexIndex)
			{
				if (FVec3::IsNearlyEqual(GetVertex(LocalVertexIndex), InPos, InTolerance))
				{
					return LocalVertexIndex;
				}
			}
			return INDEX_NONE;
		};

		int32 GetVertexIDAt(const FVec3& InPos, const FReal InTolerance) const
		{
			const int32 LocalVertexIndex = GetLocalVertexIndexAt(InPos, InTolerance);
			if (LocalVertexIndex != INDEX_NONE)
			{
				return VertexIndices[LocalVertexIndex];
			}
			return INDEX_NONE;
		};

		const FTriangle& GetTriangle() const
		{
			return Triangle;
		}

		// Get the vertex for the triangle-local vertex index [0,2]
		const FVec3& GetVertex(const int32 LocalVertexIndex) const
		{
			return Triangle.GetVertex(LocalVertexIndex);
		}

		int32 GetTriangleIndex() const
		{
			return TriangleIndex;
		}

		int32 GetVertexIndex(const int32 LocalIndex) const
		{
			return VertexIndices[LocalIndex];
		}

		const FVec3& GetNormal() const
		{
			if (Normal.X == InvalidNormalMarker)
			{
				Normal = Triangle.GetNormal();
			}
			return Normal;
		}

		FVec3 GetCentroid() const
		{
			return Triangle.GetCentroid();
		}

		void SetVisitIndex(const int8 InVisitIndex)
		{
			VisitIndex = InVisitIndex;
		}

		int8 GetVisitIndex() const
		{
			return VisitIndex;
		}

		void SetEnabled(const bool bInEnabled)
		{
			bEnabled = bInEnabled;
		}

		bool GetIsEnabled() const
		{
			return bEnabled;
		}

		void AddFaceEdgeCollision()
		{
			++NumFaceEdgeCollisions;
		}

		int32 GetNumFaceEdgeCollisions() const
		{
			return NumFaceEdgeCollisions;
		}

	private:
		FTriangle Triangle;
		mutable FVec3 Normal;
		int32 TriangleIndex;
		int32 VertexIndices[3];
		int8 NumFaceEdgeCollisions;
		int8 VisitIndex;
		bool bEnabled;
	};

	/**
	* Generate contacts between a collision shape and the triangles from a mesh.
	*/
	class FMeshContactGenerator
	{
	public:
		FMeshContactGenerator(const FMeshContactGeneratorSettings& InSettings);

		// Enable or disable the normal fixup
		void SetFixNormalsEnabled(const bool bInFixNormals)
		{
			Settings.bFixNormals = bInFixNormals;
		}

		// Clear and initialize buffers
		void BeginCollect(const int32 InNumTriangles)
		{
			const int32 ExpectedNumContacts = FMath::Min(InNumTriangles * 4, Settings.MaxContactsBufferSize);
			Reset(InNumTriangles, ExpectedNumContacts);
		}

		// Add a triangle that we might be overlapping
		void AddTriangle(const FTriangle& MeshTriangle, const int32 MeshTriangleIndex, const int32 VertexIndex0, const int32 VertexIndex1, const int32 VertexIndex2)
		{
			Triangles.Emplace(MeshTriangle, MeshTriangleIndex, VertexIndex0, VertexIndex1, VertexIndex2);
		}

		// Process all the added triangles to generate connectivity metadata etc
		void EndCollect()
		{
			for (int32 LocalTriangleIndex = 0; LocalTriangleIndex < GetNumTriangles(); ++LocalTriangleIndex)
			{
				const FTriangleExt& Triangle = Triangles[LocalTriangleIndex];
				AddTriangleEdge(LocalTriangleIndex, Triangle.GetVertexIndex(0), Triangle.GetVertexIndex(1));
				AddTriangleEdge(LocalTriangleIndex, Triangle.GetVertexIndex(1), Triangle.GetVertexIndex(2));
				AddTriangleEdge(LocalTriangleIndex, Triangle.GetVertexIndex(2), Triangle.GetVertexIndex(0));
			}
		}

		// Loop over (the required subset of) all triangles and call the TriangleContactGenerator to create a manifold for each.
		// TriangleContactGeneratorType: void(const FTriangle& Triangle, FContactPointManifold& OutContactPoints)
		template<typename TriangleContactGeneratorType>
		void GenerateMeshContacts(const TriangleContactGeneratorType& TriangleContactGenerator)
		{
			if (!!Settings.bUseTwoPassLoop)
			{
				GenerateMeshContactsTwoPass<TriangleContactGeneratorType>(TriangleContactGenerator);
			}
			else
			{
				GenerateMeshContactsOnePass<TriangleContactGeneratorType>(TriangleContactGenerator);
			}
		}

		// Process all the contact points generated by GenerateMeshContacts. This prunes duplicates, fixes normals, and
		// transforms the contact data back into shape-local space.
		void ProcessGeneratedContacts(const FRigidTransform3& ConvexTransform, const FRigidTransform3& MeshToConvexTransform);

		// The results of contact generation (must call ProcessGeneratedContacts prior to GetContactPoints)
		TArrayView<const FContactPoint> GetContactPoints() const
		{
			return MakeArrayView(Contacts);
		}

	private:
		using FTriangleExt = FMeshContactGeneratorTriangle;

		// Info about a vertex contact that allows us to prevent adding duplicates from all the triangles that 
		// share the vertex. Contains a contact index combined with a flag to indicate if the normal is roughly 
		// along the triangle face
		struct FVertexContactData
		{
			FVertexContactData()
			{
			}
			
			FVertexContactData(const FContactVertexID& InID, const int32 InContactIndex, const bool bInIsFaceContact)
				: ID(InID)
				, ContactIndex(InContactIndex)
				, bIsFaceContact(bInIsFaceContact)
			{
			}

			FContactVertexID ID;
			int32 ContactIndex;
			bool bIsFaceContact;
		};

		// Info about an edge contact that allows us to prevent adding duplicate edge contacts from the two
		// triangles that share the edge.
		struct FEdgeContactData
		{
			FEdgeContactData()
			{
			}

			FEdgeContactData(const FContactEdgeID& InID, const int32 InLocalTriangleIndex)
			{
				Init(InID, InLocalTriangleIndex);
			}

			void Init(const FContactEdgeID& InID, const int32 InLocalTriangleIndex)
			{
				ID = InID;
				LocalTriangleIndex = InLocalTriangleIndex;
				ContactIndices[0] = INDEX_NONE;
				ContactIndices[1] = INDEX_NONE;
				ContactNormalDotTriangleNormal = 0.0f;
				bIsFaceContact = false;
			}

			void AddContact(int32 InContactIndex, const FRealSingle InContactNormalDotTriangleNormal, const bool bInIsFaceContact)
			{
				if (ContactIndices[0] == INDEX_NONE)
				{
					ContactIndices[0] = InContactIndex;
				}
				else
				{
					ContactIndices[1] = InContactIndex;
				}
				ContactNormalDotTriangleNormal = FMath::Max(ContactNormalDotTriangleNormal, InContactNormalDotTriangleNormal);
				bIsFaceContact = bIsFaceContact || bInIsFaceContact;
			}

			FContactEdgeID ID;
			int32 LocalTriangleIndex;
			int32 ContactIndices[2];
			FRealSingle ContactNormalDotTriangleNormal;
			bool bIsFaceContact;
		};

		// The triangle indices that share an edge (assumes only 2)
		struct FEdgeTriangleIndices
		{
			FEdgeTriangleIndices(const FContactEdgeID& InEdgeID, const int32 InIndex0, const int32 InIndex1)
				: ID(InEdgeID)
				, LocalTriangleIndices{ InIndex0, InIndex1 }
			{
			}

			FContactEdgeID ID;
			int32 LocalTriangleIndices[2];
		};

		template<typename TriangleContactGeneratorType>
		void GenerateMeshContactsOnePass(const TriangleContactGeneratorType& TriangleContactGenerator)
		{
			for (int32 LocalTriangleIndex = 0; LocalTriangleIndex < GetNumTriangles(); ++LocalTriangleIndex)
			{
				TriangleContactGenerator(*this, LocalTriangleIndex);

				SetTriangleVisited(LocalTriangleIndex, 0);
			}
		}

		template<typename TriangleContactGeneratorType>
		void GenerateMeshContactsTwoPass(const TriangleContactGeneratorType& TriangleContactGenerator)
		{
			// First loop: Visit triangles that do not have any collisions on any of their vertices or edges.
			// This will skip all triangles whose neighbours have already been processed and generated a contact
			// on a shared edge/vertex.
			for (int32 LocalTriangleIndex = 0; LocalTriangleIndex < GetNumTriangles(); ++LocalTriangleIndex)
			{
				if (GetNumTriangleFaceCollisions(LocalTriangleIndex) == 0)
				{
					TriangleContactGenerator(*this, LocalTriangleIndex);

					SetTriangleVisited(LocalTriangleIndex, 0);
				}
			}

			// Second loop: Visit remaining triangles that have less than 3 contacts on them. This will skip all triangles
			// that have a full manifold as a result of collisions on shared edges/vertices from adjacent triangles.
			for (int32 LocalTriangleIndex = 0; LocalTriangleIndex < GetNumTriangles(); ++LocalTriangleIndex)
			{
				if (!IsTriangleVisited(LocalTriangleIndex) && (GetNumTriangleFaceCollisions(LocalTriangleIndex) < 3))
				{
					TriangleContactGenerator(*this, LocalTriangleIndex);

					SetTriangleVisited(LocalTriangleIndex, 1);
				}
			}
		}

		void Reset(const int32 InMaxTriangles, const int32 InMaxContacts);

		void AddTriangleEdge(const int32 LocalTriangleIndex, const int32 VertexIndex0, const int32 VertexIndex1)
		{
			const FContactEdgeID EdgeID = FContactEdgeID(VertexIndex0, VertexIndex1);
			FEdgeTriangleIndices* EdgeTriangleIndices = EdgeTriangleIndicesMap.Find(EdgeID);
			if (EdgeTriangleIndices == nullptr)
			{
				EdgeTriangleIndicesMap.Emplace(EdgeID, EdgeID, LocalTriangleIndex, INDEX_NONE);
			}
			else
			{
				EdgeTriangleIndices->LocalTriangleIndices[1] = LocalTriangleIndex;
			}
		}

		bool IsSharedEdge(const FContactEdgeID& EdgeID) const
		{
			const FEdgeTriangleIndices* EdgeTriangleIndices = EdgeTriangleIndicesMap.Find(EdgeID);
			if (EdgeTriangleIndices != nullptr)
			{
				return (EdgeTriangleIndices->LocalTriangleIndices[0] != INDEX_NONE) && (EdgeTriangleIndices->LocalTriangleIndices[1] != INDEX_NONE);
			}
			return false;
		}

		int32 GetOtherTriangleIndexForEdge(const int32 LocalTriangleIndex, const FContactEdgeID& EdgeID)
		{
			FEdgeTriangleIndices* EdgeTriangleIndices = EdgeTriangleIndicesMap.Find(EdgeID);
			if (EdgeTriangleIndices != nullptr)
			{
				return (EdgeTriangleIndices->LocalTriangleIndices[0] == LocalTriangleIndex) ? EdgeTriangleIndices->LocalTriangleIndices[1] : EdgeTriangleIndices->LocalTriangleIndices[0];
			}
			return INDEX_NONE;
		}

		bool HasFaceVertexCollision(const FContactVertexID VertexID) const
		{
			if (const FVertexContactData* ContactData = VertexContactDataMap.Find(VertexID))
			{
				return ContactData->bIsFaceContact;
			}
			return false;
		}

		int32 GetNumTriangleFaceCollisions(const int32 LocalTriangleIndex) const
		{
			const int32 MeshVertexIndex0 = Triangles[LocalTriangleIndex].GetVertexIndex(0);
			const int32 MeshVertexIndex1 = Triangles[LocalTriangleIndex].GetVertexIndex(1);
			const int32 MeshVertexIndex2 = Triangles[LocalTriangleIndex].GetVertexIndex(2);
			int32 NumCollisions = Triangles[LocalTriangleIndex].GetNumFaceEdgeCollisions();
			NumCollisions += HasFaceVertexCollision(MeshVertexIndex0) ? 1 : 0;
			NumCollisions += HasFaceVertexCollision(MeshVertexIndex1) ? 1 : 0;
			NumCollisions += HasFaceVertexCollision(MeshVertexIndex2) ? 1 : 0;
			return NumCollisions;
		}

		bool IsTriangleVisited(const int32 LocalTriangleIndex) const
		{
			return Triangles[LocalTriangleIndex].GetVisitIndex() != INDEX_NONE;
		}

		void SetTriangleVisited(const int32 LocalTriangleIndex, const int8 VisitIndex)
		{
			Triangles[LocalTriangleIndex].SetVisitIndex(VisitIndex);
		}

	public:
		int32 GetNumTriangles() const
		{
			return Triangles.Num();
		}

		const FTriangle& GetTriangle(const int32 LocalTriangleIndex) const
		{
			return Triangles[LocalTriangleIndex].GetTriangle();
		}

		const FVec3& GetTriangleNormal(const int32 LocalTriangleIndex) const
		{
			return Triangles[LocalTriangleIndex].GetNormal();
		}

		bool FixFeature(const int32 LocalTriangleIndex, Private::EConvexFeatureType& InOutFeatureType, int32& InOutFeatureIndex, FVec3& InOutPlaneNormal);

		void AddTriangleContacts(const int32 LocalTriangleIndex, const TArrayView<FContactPoint>& TriangleContactPoints);

	private:
		void PruneAndCorrectContacts();
		void FixContactNormal(const int32 ContactIndex);
		void RemoveDisabledContacts();
		void SortContactByPhi();
		void SortContactsForSolverConvergence();
		void FinalizeContacts(const FRigidTransform3& MeshToConvexTransform);

		void DebugDrawContacts(const FRigidTransform3& ConvexTransform, const FColor& Color, const FReal LineScale);
		void DebugDrawTriangles(const FRigidTransform3& ConvexTransform, const FColor& VisitedColor, const FColor& IgnoredColor);
		void DebugDrawTriangle(const FRigidTransform3& ConvexTransform, const FTriangleExt& TriangleData, const FColor& Color);

	private:
		FMeshContactGeneratorSettings Settings;

		// All the triangles we might collide with
		TArray<FTriangleExt> Triangles;

		// The contact data. This is split into the output data (FContactPoint) and the extra
		// per-contact metadata required during processing (FTriangleContactPointData).
		TArray<FContactPoint> Contacts;
		TArray<FTriangleContactPointData> ContactDatas;

		// A map of EdgeID to the two triangles (indices) that use the Edge
		struct FEdgeTriangleIndicesMapTraits
		{
			static uint32 GetIDHash(const FContactEdgeID& EdgeID) { return uint32(MurmurFinalize64(EdgeID.EdgeID)); }
			static FContactEdgeID GetElementID(const FEdgeTriangleIndices& TriangleIndices) { return TriangleIndices.ID; }
		};
		THashMappedArray<FContactEdgeID, FEdgeTriangleIndices, FEdgeTriangleIndicesMapTraits> EdgeTriangleIndicesMap;

		// A map of VertexID to contact data for that vertex - we only ever keep one contact per vertex
		struct FVertexContactIndicesMapTraits
		{
			static uint32 GetIDHash(const FContactVertexID& VertexID) { return MurmurFinalize32(VertexID); }
			static FContactVertexID GetElementID(const FVertexContactData& VertexContactData) { return VertexContactData.ID; }
		};
		THashMappedArray<FContactVertexID, FVertexContactData, FVertexContactIndicesMapTraits> VertexContactDataMap;

		// A map of EdgeID to contact data for that edge - we only ever keep contacts from one triangle sharing the edge
		struct FEdgeContactIndicesMapTraits
		{
			static uint32 GetIDHash(const FContactEdgeID& EdgeID) { return uint32(MurmurFinalize64(EdgeID.EdgeID)); }
			static FContactEdgeID GetElementID(const FEdgeContactData& EdgeContactData) { return EdgeContactData.ID; }
		};
		THashMappedArray<FContactEdgeID, FEdgeContactData, FEdgeContactIndicesMapTraits> EdgeContactDataMap;
	};

}