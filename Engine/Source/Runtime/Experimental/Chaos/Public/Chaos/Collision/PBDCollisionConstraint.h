// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"

#include "Chaos/BVHParticles.h"
#include "Chaos/CollisionResolutionTypes.h"
#include "Chaos/Collision/CollisionKeys.h"
#include "Chaos/Collision/ContactPoint.h"
#include "Chaos/Collision/PBDCollisionConstraintHandle.h"
#include "Chaos/Framework/UncheckedArray.h"
#include "Chaos/GJK.h"
#include "Chaos/ParticleHandleFwd.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/Vector.h"

class FChaosVDDataWrapperUtils;

namespace Chaos
{
	namespace Private
	{
		class FCollisionConstraintAllocator;
		class FCollisionContextAllocator;
	}
	class FConstGenericParticleHandle;
	class FImplicitObject;
	class FParticlePairMidPhase;
	class FPBDCollisionConstraint;
	class FPBDCollisionConstraints;
	class FPerShapeData;
	class FShapeInstance;
	class FSingleShapePairCollisionDetector;
	class FSolverBody;
	class FSolverBodyContainer;

	UE_DEPRECATED(4.27, "Use FPBDCollisionConstraint instead")
	typedef FPBDCollisionConstraint FRigidBodyPointContactConstraint;

	CHAOS_API bool ContactConstraintSortPredicate(const FPBDCollisionConstraint& L, const FPBDCollisionConstraint& R);

	extern int32 Chaos_Collision_MaxManifoldPoints;

	/*
	 * @brief Material properties for a collision constraint
	*/
	class FPBDCollisionConstraintMaterial
	{
	public:

		// NOTE: This pragma is needed until the deprecated properties are removed
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FPBDCollisionConstraintMaterial()
			: FaceIndex(INDEX_NONE)
			, DynamicFriction(0)
			, StaticFriction(0)
			, Restitution(0)
			, RestitutionThreshold(0)
			, InvMassScale0(1)
			, InvMassScale1(1)
			, InvInertiaScale0(1)
			, InvInertiaScale1(1)
			, SoftSeparation(0)
			, BaseFrictionImpulse(0)
		{ }

		FPBDCollisionConstraintMaterial(const FPBDCollisionConstraintMaterial& Other)
			: FaceIndex(Other.FaceIndex)
			, DynamicFriction(Other.DynamicFriction)
			, StaticFriction(Other.StaticFriction)
			, Restitution(Other.Restitution)
			, RestitutionThreshold(Other.RestitutionThreshold)
			, InvMassScale0(Other.InvMassScale0)
			, InvMassScale1(Other.InvMassScale1)
			, InvInertiaScale0(Other.InvInertiaScale0)
			, InvInertiaScale1(Other.InvInertiaScale1)
			, SoftSeparation(Other.SoftSeparation)
			, BaseFrictionImpulse(Other.BaseFrictionImpulse)
		{ }

		FPBDCollisionConstraintMaterial& operator=(const FPBDCollisionConstraintMaterial& Other)
		{
			FaceIndex = Other.FaceIndex;
			DynamicFriction = Other.DynamicFriction;
			StaticFriction = Other.StaticFriction;
			Restitution = Other.Restitution;
			RestitutionThreshold = Other.RestitutionThreshold;
			InvMassScale0 = Other.InvMassScale0;
			InvMassScale1 = Other.InvMassScale1;
			InvInertiaScale0 = Other.InvInertiaScale0;
			InvInertiaScale1 =Other.InvInertiaScale1;
			SoftSeparation = Other.SoftSeparation;
			BaseFrictionImpulse = Other.BaseFrictionImpulse;
			return *this;
		}

		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		// The face index that the material was extracted from
		int32 FaceIndex;

		// Final material properties (post modifier) used by the solver
		// These get reset every frame to the material values (if modified)
		FRealSingle DynamicFriction;
		FRealSingle StaticFriction;
		FRealSingle Restitution;
		FRealSingle RestitutionThreshold;
		FRealSingle InvMassScale0;
		FRealSingle InvMassScale1;
		FRealSingle InvInertiaScale0;
		FRealSingle InvInertiaScale1;

		UE_DEPRECATED(5.5, "Use DynamicFriction instead")
		FRealSingle MaterialDynamicFriction;
		UE_DEPRECATED(5.5, "Use StaticFriction instead")
		FRealSingle MaterialStaticFriction;
		UE_DEPRECATED(5.5, "Use Restitution instead")
		FRealSingle MaterialRestitution;

		// SoftSeparation is usually negative (0 for disabled), and indicates the depth at which the hard collision starts
		FRealSingle SoftSeparation;

		// BaseFrictionThickness is usually positive (0 for disabled) and is the distance away from the core shape
		// at which the base friction impulse can be applied.
		FRealSingle BaseFrictionImpulse;
	};

	// Renamed to FPBDCollisionConstraintMaterial
	using FCollisionContact UE_DEPRECATED(5.1, "FCollisionContact was renamed to FPBDCollisionConstraintMaterial") = FPBDCollisionConstraintMaterial;

	namespace Private
	{

		// Flags to specify what types of bounds tests should be run for a collision constraint
		// NOTE: These are separate from the other constraint flags because the ShapePair MidPhase 
		// stores a copy for use in determining whether to create a constraint in the first place.
		union FImplicitBoundsTestFlags
		{
			FImplicitBoundsTestFlags() : Bits(0) {}
			struct
			{
				uint8 bEnableAABBCheck : 1;
				uint8 bEnableOBBCheck0 : 1;
				uint8 bEnableOBBCheck1 : 1;
				uint8 bEnableDistanceCheck : 1;
				uint8 bEnableManifoldUpdate : 1;
				uint8 bIsProbe : 1;
			};
			uint8 Bits;
		};

		CHAOS_API FImplicitBoundsTestFlags CalculateImplicitBoundsTestFlags(
			FGeometryParticleHandle* Particle0, const FImplicitObject* Implicit0, const FPerShapeData* Shape0,
			FGeometryParticleHandle* Particle1, const FImplicitObject* Implicit1, const FPerShapeData* Shape1,
			FRealSingle& OutDistanceCheckSize);
	}

	/**
	 * @brief Information used by the constraint allocator
	 * This includes any information used for optimizations like array indexes etc
	 * as well as data for managing lifetime and pruning.
	*/
	class FPBDCollisionConstraintContainerCookie
	{
	public:
		FPBDCollisionConstraintContainerCookie()
			: MidPhase(nullptr)
			, CreationEpoch(INDEX_NONE)
			, LastUsedEpoch(INDEX_NONE)
			, ConstraintIndex(INDEX_NONE)
			, bIsMultiShapePair(false)
			, CCDConstraintIndex(INDEX_NONE)
		{
		}

		/**
		 * @brief Used to clear the container data when copying constraints out of the container (see resim cache)
		 * The constraint index will not be valid when the constraint is copied out of the container, but everything
		 * else is ok and should be restorable.
		*/
		void ClearContainerData()
		{
			MidPhase = nullptr;
			ConstraintIndex = INDEX_NONE;
			CCDConstraintIndex = INDEX_NONE;
		}

		// The constraint owner - set when the constraint is created
		FParticlePairMidPhase* MidPhase;

		// The Epoch when then constraint was initially created
		int32 CreationEpoch;

		// The Epoch when the constraint was last used
		int32 LastUsedEpoch;

		// The index in the container - this changes every tick (is valid for all constraints, including CCD)
		int32 ConstraintIndex : 31;

		// Used by the MidPhase when a constraint is reactivated from a Resim cache
		// If true, indicates that the constraint was created from the recursive collision detection
		// path rather than the prefiltered shape-pair loop
		uint32 bIsMultiShapePair : 1;

		// The CCD index in the container - this changes every tick (is INDEX_NONE for non-CCD constraints)
		int32 CCDConstraintIndex;
	};


	/**
	 * @brief A contact constraint
	 * 
	 * A contact constraint represents the non-penetration, friction, and restitution constraints for a single
	 * shape pair on a particle pair. I.e., a specific particle-pair may have multiple contact constraints
	 * between them if one or boht has multuple collision shapes that overlap the shape(s) of the other body.
	 * 
	 * Each contact constraint contains a Manifold, which is a set of contact points that approximate the
	 * contact patch between the two shapes.
	 * 
	 * Contact constraints are allocated on the heap (Asee FCollisionConstraintAllocator) and have permanent addresses. 
	 * They use intrusive handles to reduce unnecessary indirection.
	 * 
	*/
	class FPBDCollisionConstraint final : public FPBDCollisionConstraintHandle
	{
		friend class Private::FCollisionConstraintAllocator;
		friend class Private::FCollisionContextAllocator;
		friend class FContactPairModifier;
		friend class FGenericParticlePairMidPhase;
		friend class FParticlePairMidPhase;
		friend class FPBDCollisionConstraints;
		friend class FShapePairParticlePairMidPhase;
		friend class FSingleShapePairCollisionDetector;
		friend class FSphereApproximationParticlePairMidPhase;

		friend CHAOS_API bool ContactConstraintSortPredicate(const FPBDCollisionConstraint& L, const FPBDCollisionConstraint& R);

	public:
		using FConstraintContainerHandle = TIntrusiveConstraintHandle<FPBDCollisionConstraint>;

		static constexpr FRealSingle MaxTOI = std::numeric_limits<FRealSingle>::max();

		/**
		 * @brief Create a contact constraint
		 * Initializes a constraint stored inline in an object. Only intended to be called once right after construction.
		 * Does not reinitialize all data so not intended to reset a constraint for reuse with different particles etc.
		*/
		static CHAOS_API void Make(
			FGeometryParticleHandle* Particle0,
			const FImplicitObject* Implicit0,
			const FPerShapeData* Shape0,
			const FBVHParticles* Simplicial0,
			const FRigidTransform3& ImplicitLocalTransform0,
			FGeometryParticleHandle* Particle1,
			const FImplicitObject* Implicit1,
			const FPerShapeData* Shape1,
			const FBVHParticles* Simplicial1,
			const FRigidTransform3& ImplicitLocalTransform1,
			const FReal InCullDistance,
			const bool bInUseManifold,
			const EContactShapesType ShapesType,
			FPBDCollisionConstraint& OutConstraint);

		/**
		 * @brief For use by the tri mesh and heighfield collision detection as a temporary measure
		 * @see FHeightField::ContactManifoldImp, FTriangleMeshImplicitObject::ContactManifoldImp
		*/
		static CHAOS_API FPBDCollisionConstraint MakeTriangle(const FImplicitObject* Implicit0);

		/**
		 * @brief Return a constraint copied from the Source constraint, for use in the Resim Cache or other system
		 * @note Unlike the other factory method, this version returns a constraint by value for emplacing into an array (ther others are by pointer)
		*/
		static CHAOS_API FPBDCollisionConstraint MakeCopy(const FPBDCollisionConstraint& Source);

		/**
		 * Restore the properties of a collision from the properties of another. Used by the rewind/resim system when restoring
		 * a collision constraint that was saved (and called MakeCopy).
		 * This takes care not to overwrite any data maintained for the use of other systems (the ContainerCookie, Graph data, etc)
		 */
		CHAOS_API void RestoreFrom(const FPBDCollisionConstraint& Source);

		FPBDCollisionConstraint();
		CHAOS_API virtual ~FPBDCollisionConstraint();

		/**
		 * Whether CCD is enabled for this collision.
		 * This value depends only on the CCD requirements of the two particles. It does not change from tick to tick.
		 */
		bool GetCCDEnabled() const { return Flags.bCCDEnabled; }

		/**
		 * Enable or disable CCD for this constraint. Called once right after the constraint is created.
		 */
		void SetCCDEnabled(const bool bCCDEnabled)
		{ 
			Flags.bCCDEnabled = bCCDEnabled;
			Flags.bCCDSweepEnabled = bCCDEnabled;

			// Initialize the CCD thresholds (one time only)
			if (bCCDEnabled && (CCDEnablePenetration == FReal(0)))
			{
				InitCCDThreshold();
			}
		}

		/**
		 * For CCD constraints, a transient flag (reset every frame) that is used to both enable/disable CCD sweeps for
		 * the frame, and to report whether the sweep was run and the results retained. It will be set to false if
		 * the bodies are moving less than the CCD threshold this tick.
		 * 
		 * If this is true in a collision callback then: 1) a CCD sweep was performed; 2) a CCD impact was detected; and
		 * 3) the collision is being treated as a CCD contact. The bodies will be moved back to first CCD contact.
		 * 
		 * NOTE: If a sweep is performed and a hit detected, but the resultant contact is shallow enough to be
		 * handled by the standard collision solver path, this flag is set to false and no rollback will be performed.
		 */
		bool GetCCDSweepEnabled() const { return Flags.bCCDSweepEnabled; }

		/**
		 * For CCD constraints, a transient flag (reset every frame) that is used to both enable/disable CCD sweeps for
		 * the frame, and to retport whether the sweep was run and the results retained. See GetCCDSweepEnabled().
		 */
		void SetCCDSweepEnabled(const bool bCCDSweepEnabled)
		{
			Flags.bCCDSweepEnabled = bCCDSweepEnabled;
		}

		/**
		 * @brief If CCD is enabled, contacts deeper than this will be handled by CCD
		*/
		FReal GetCCDEnablePenetration() const
		{
			return CCDEnablePenetration;
		}

		/**
		 * @brief If CCD is enabled and processed the contact, CCD resolution leaves up to this much penetration
		*/
		FReal GetCCDTargetPenetration() const
		{
			return CCDTargetPenetration;
		}

		//
		// API
		//

		FGeometryParticleHandle* GetParticle0() const { return Particle[0]; }
		FGeometryParticleHandle* GetParticle1() const { return Particle[1]; }
		FGeometryParticleHandle* GetParticle(const int32 ParticleIndex) const { check((ParticleIndex >= 0) && (ParticleIndex < 2)); return Particle[ParticleIndex]; }

		const FImplicitObject* GetImplicit0() const { return Implicit[0]; }
		const FImplicitObject* GetImplicit1() const { return Implicit[1]; }
		const FImplicitObject* GetImplicit(const int32 ParticleIndex) const { check((ParticleIndex >= 0) && (ParticleIndex < 2)); return Implicit[ParticleIndex]; }

		const FShapeInstance* GetShape0() const { return Shape[0]; }
		const FShapeInstance* GetShape1() const { return Shape[1]; }
		const FShapeInstance* GetShape(const int32 ParticleIndex) const { check((ParticleIndex >= 0) && (ParticleIndex < 2)); return Shape[ParticleIndex]; }

		const FBVHParticles* GetCollisionParticles0() const { return Simplicial[0]; }
		const FBVHParticles* GetCollisionParticles1() const { return Simplicial[1]; }
		const FBVHParticles* GetCollisionParticles(const int32 ParticleIndex) const { check((ParticleIndex >= 0) && (ParticleIndex < 2)); return Simplicial[ParticleIndex]; }

		const FReal GetCollisionMargin0() const { return CollisionMargins[0]; }
		const FReal GetCollisionMargin1() const { return CollisionMargins[1]; }

		const bool IsQuadratic0() const { return Flags.bIsQuadratic0; }
		const bool IsQuadratic1() const { return Flags.bIsQuadratic1; }
		const bool HasQuadraticShape() const { return (Flags.bIsQuadratic0 || Flags.bIsQuadratic1); }
		const FReal GetCollisionRadius0() const { return (Flags.bIsQuadratic0) ? CollisionMargins[0] : FReal(0); }
		const FReal GetCollisionRadius1() const { return (Flags.bIsQuadratic1) ? CollisionMargins[1] : FReal(0); }

		/** 
		 * Called each frame when the constraint is active after primary collision detection (but not per incremental collision detection call if enabled) 
		 */
		CHAOS_API void Activate();

		UE_DEPRECATED(5.2, "Removed parameter")
		void Activate(const FReal Dt) { Activate(); }

		// When a particle is moved under user control, we need to update some cached state to prevent friction from undoing the move
		CHAOS_API void UpdateParticleTransform(FGeometryParticleHandle* InParticle);

		// @todo(chaos): half of this API is wrong for the new multi-point manifold constraints. Remove it

		void ResetPhi(FReal InPhi) { ClosestManifoldPointIndex = INDEX_NONE; }
		FReal GetPhi() const { return (ClosestManifoldPointIndex != INDEX_NONE) ? ManifoldPoints[ClosestManifoldPointIndex].ContactPoint.Phi : TNumericLimits<FReal>::Max(); }

		// A tick reset called on all constraints that were Activated last tick
		void BeginTick()
		{
			Flags.bDisabled = true;
			if (!IsSleeping())
			{
				Flags.bIsCurrent = false;
			}
			ContainerCookie.ConstraintIndex = INDEX_NONE;
			ContainerCookie.CCDConstraintIndex = INDEX_NONE;
		}

		// Was this constraint activated this frame? It will be activated if the shapes are within CullDistance of each other.
		// NOTE: All Awake Current constraints are in the ActiveConstraints list on the CollisionConstraintAllocator. This remains true 
		// even if disabled by the user (via SetDisabled()) in a callback. We usually only care about "not current" constraints
		// for debug visualization/reporting. Normally you would only need to consider GetDisabled()
		// NOTE: sleeping constraints are also considered Current, but are not in the ActiveConstraints list
		bool IsCurrent() const { return Flags.bIsCurrent; }

		// Allow the user to disable this constraint. @see GetActive().
		void SetDisabled(bool bInDisabled) { Flags.bDisabled = bInDisabled; }

		// Whether this constraint was disabled by the user (e.g., via a collision callback)
		bool GetDisabled() const { return Flags.bDisabled; }

		bool GetIsOneWayInteraction() const { return Flags.bIsOneWayInteraction; }

		void SetIsProbe(bool bInProbe) { Flags.bIsProbe = bInProbe; }
		bool GetIsProbe() const { return Flags.bIsProbe; }

		// Is this considered an initial contact (i.e., contact was activated this tick, but not last)
		void SetIsInitialContact(const bool bInIsInitialContact) { Flags.bInitialContact = bInIsInitialContact; }
		bool IsInitialContact() const { return Flags.bInitialContact; }
		FRealSingle GetMinInitialPhi() const { return MinInitialPhi; }
		bool UsePerContactInitialPhi() const { return Flags.bUsePerContactInitialPhi; }

		virtual bool SupportsSleeping() const override final { return true; }
		virtual bool SupportsPartialIslandSleeping() const override final { return true; }
		CHAOS_API virtual bool IsSleeping() const override final;
		CHAOS_API virtual void SetIsSleeping(const bool bInIsSleeping) override final;

		// The total impulse for a collision constraint is computed summing over the net impulse of all manifold points.
		CHAOS_API virtual FVec3f GetNetLinearImpulse() const override final;

		// Get the world-space normal of the closest manifold point
		// @todo(chaos): remove (used by legacy RBAN collision solver)
		CHAOS_API FVec3 CalculateWorldContactNormal() const;

		// Get the world-space contact location of the closest manifold point
		// @todo(chaos): remove (used by legacy RBAN collision solver)
		CHAOS_API FVec3 CalculateWorldContactLocation() const;

		// Called to force the contact to recollect its material properties (e.g., when materials are modified)
		void ClearMaterialProperties()
		{
			Flags.bMaterialSet = false;
		}

		// Find the material for this contact and collect the friction, restitution, etc. This data is cached in the 
		// contact, so ClearMaterialProperties must be called if the material changes to force a re-gather of the data.
		void UpdateMaterialProperties()
		{
			if (!Flags.bMaterialSet)
			{
				UpdateMaterialPropertiesImpl();
				Flags.bMaterialSet = true;
			}
		}

		void SetModifierApplied() { Flags.bModifierApplied = true; }

		UE_DEPRECATED(5.5, "Use specific material property getters instead")
		const FPBDCollisionConstraintMaterial& GetCollisionMaterial() const { return Material; }

		void SetInvMassScale0(const FReal InInvMassScale) { Material.InvMassScale0 = FRealSingle(InInvMassScale); }
		FReal GetInvMassScale0() const { return Material.InvMassScale0; }

		void SetInvMassScale1(const FReal InInvMassScale) { Material.InvMassScale1 = FRealSingle(InInvMassScale); }
		FReal GetInvMassScale1() const { return Material.InvMassScale1; }

		void SetInvInertiaScale0(const FReal InInvInertiaScale) { Material.InvInertiaScale0 = FRealSingle(InInvInertiaScale); }
		FReal GetInvInertiaScale0() const { return Material.InvInertiaScale0; }

		void SetInvInertiaScale1(const FReal InInvInertiaScale) { Material.InvInertiaScale1 = FRealSingle(InInvInertiaScale); }
		FReal GetInvInertiaScale1() const { return Material.InvInertiaScale1; }

		void SetStiffness(FReal InStiffness) { Stiffness = FRealSingle(InStiffness); }
		FReal GetStiffness() const { return Stiffness; }

		void SetRestitution(const FReal InRestitution) { Material.Restitution = FRealSingle(InRestitution); }
		FReal GetRestitution() const { return Material.Restitution; }

		void SetRestitutionThreshold(const FReal InRestitutionThreshold) { Material.RestitutionThreshold = FRealSingle(InRestitutionThreshold); }
		FReal GetRestitutionThreshold() const { return Material.RestitutionThreshold; }

		void SetStaticFriction(const FReal InStaticFriction) { Material.StaticFriction = FRealSingle(InStaticFriction); }
		FReal GetStaticFriction() const { return FMath::Max(Material.StaticFriction, Material.DynamicFriction); }

		void SetDynamicFriction(const FReal InDynamicFriction) { Material.DynamicFriction = FRealSingle(InDynamicFriction); }
		FReal GetDynamicFriction() const { return Material.DynamicFriction; }

		void SetMinFrictionPushOut(const FReal InMinFrictionPushOut) { Material.BaseFrictionImpulse = FRealSingle(InMinFrictionPushOut); }
		FReal GetMinFrictionPushOut() const { return Material.BaseFrictionImpulse; }

		bool IsSoftContact() const { return (Material.SoftSeparation != 0); }

		void SetSoftSeparation(const FReal InSoftSeparation) { Material.SoftSeparation = FRealSingle(InSoftSeparation); }
		FRealSingle GetSoftSeparation() const { return Material.SoftSeparation; }

		EContactShapesType GetShapesType() const { return ShapesType; }

		CHAOS_API FString ToString() const;

		FReal GetCullDistance() const { return CullDistance; }
		FRealSingle GetCullDistancef() const { return CullDistance; }
		void SetCullDistance(FReal InCullDistance) { CullDistance = FRealSingle(InCullDistance); }

		FVec3f GetRelativeMovement() const { return RelativeMovement; }
		void SetRelativeMovement(const FVec3f& InDelta) { RelativeMovement = InDelta; }

		// Whether we are using manifolds (either one-shot or incremental)
		bool GetUseManifold() const { return Flags.bUseManifold; }

		// Whether we can use incremental manifolds (updated each iteration)
		bool GetUseIncrementalManifold() const { return Flags.bUseIncrementalManifold; }

		// Whether we run collision detection every iteration (true if we are not using one shot manifolds)
		// NOTE: This is initially set based on whether are allowing incremental manifolds
		bool GetUseIncrementalCollisionDetection() const { return !Flags.bUseManifold || Flags.bUseIncrementalManifold; }

		// Initial overlap depenetration velocity from the maximum of the two bodies
		FRealSingle GetInitialOverlapDepenetrationVelocity() const { return InitialOverlapDepenetrationVelocity; }

		/**
		* Reset the material properties to those from the shape materials. Called each frame to reset contact modifications to the material.
		*/
		inline void ResetModifications()
		{
			if (Flags.bModifierApplied)
			{
				// Re-gather the material properties
				ClearMaterialProperties();
				UpdateMaterialProperties();

				// Reset other properties which may have changed in contact modification last frame
				Flags.bIsProbe = BoundsTestFlags.bIsProbe;

				Flags.bModifierApplied = false;
			}
		}

		/**
		 * @brief Clear the current and previous manifolds
		*/
		CHAOS_API void ResetManifold();

		// @todo(chaos): remove array view and provide per-point accessor
		TArrayView<FManifoldPoint> GetManifoldPoints() { return MakeArrayView(ManifoldPoints); }
		TArrayView<const FManifoldPoint> GetManifoldPoints() const { return MakeArrayView(ManifoldPoints); }

		int32 NumManifoldPoints() const { return ManifoldPoints.Num(); }
		FManifoldPoint& GetManifoldPoint(const int32 PointIndex) { return ManifoldPoints[PointIndex]; }
		const FManifoldPoint& GetManifoldPoint(const int32 PointIndex) const { return ManifoldPoints[PointIndex]; }
		const FManifoldPoint* GetClosestManifoldPoint() const { return (ClosestManifoldPointIndex != INDEX_NONE) ? &ManifoldPoints[ClosestManifoldPointIndex] : nullptr; }
		bool IsManifoldPointActive(const int32 PointIndex) const { return IsEnabled() && (PointIndex < NumManifoldPoints()) && !ManifoldPoints[PointIndex].Flags.bDisabled; }

		const FManifoldPointResult& GetManifoldPointResult(const int32 PointIndex) const
		{
			// We may request manifold point results for points that are never simulated (e.g., they start asleep).
			// We could handle this by adding zeroed results when we add manifold points, but that's unnecessary work for
			// active constraints, and this is function currently only used for debug draw and stats. If that changes we 
			// may want to change this.
			check(PointIndex < NumManifoldPoints());
			if (PointIndex >= ManifoldPointResults.Num())
			{
				static FManifoldPointResult ZeroResult;
				return ZeroResult;
			}
			return ManifoldPointResults[PointIndex];
		}

		inline int32 NumEnabledManifoldPoints() const
		{
			int32 NumEnabled = 0;
			if (IsEnabled())
			{
				for (int32 ManifoldPointIndex = 0; ManifoldPointIndex < NumManifoldPoints(); ++ManifoldPointIndex)
				{
					if (!ManifoldPoints[ManifoldPointIndex].Flags.bDisabled)
					{
						++NumEnabled;
					}
				}
			}
			return NumEnabled;
		}

		inline void DisableManifoldPoint(const int32 DisabledManifoldPointIndex)
		{
			ManifoldPoints[DisabledManifoldPointIndex].Flags.bDisabled = true;
			if (NumEnabledManifoldPoints() == 0)
			{
				SetDisabled(true);
			}
		}

		inline void SetManifoldPointPositionSolveEnabled(const int32 ManifoldPointIndex, bool bInEnabled)
		{
			ManifoldPoints[ManifoldPointIndex].Flags.bDisablePositionSolve = !bInEnabled;
		}

		inline void SetManifoldPointVelocitySolveEnabled(const int32 ManifoldPointIndex, bool bInEnabled)
		{
			ManifoldPoints[ManifoldPointIndex].Flags.bDisableVelocitySolve = !bInEnabled;
		}

		inline void SetManifoldPointFrictionEnabled(const int32 ManifoldPointIndex, bool bInEnabled)
		{
			ManifoldPoints[ManifoldPointIndex].Flags.bDisableFriction = !bInEnabled;
		}

		CHAOS_API void AddIncrementalManifoldContact(const FContactPoint& ContactPoint);

		// @todo(chaos): remove this and use SetOneShotManifoldContacts
		inline void AddOneshotManifoldContact(const FContactPointf& ContactPoint)
		{
			if (ContactPoint.IsSet())
			{
				int32 ManifoldPointIndex = AddManifoldPoint(ContactPoint);
				if (ManifoldPoints[ManifoldPointIndex].ContactPoint.Phi < GetPhi())
				{
					ClosestManifoldPointIndex = ManifoldPointIndex;
				}
			}
		}

		/**
		 * @brief Replace the current manifold points with the input.
		 * The input array should contain no more than MaxManifoldPoints contacts (any extra will be ignored).
		 * We assume that all input contacts have been initialized and will not return false from IsSet().
		 * Ignores contacts deeper than the CullDistance for this constraint.
		*/
		inline void SetOneShotManifoldContacts(const TArrayView<const FContactPoint>& ContactPoints)
		{
			ResetActiveManifoldContacts();

			FReal MinPhi = TNumericLimits<FReal>::Max();
			
			int32 NumContacts = ContactPoints.Num();

			// If we have too many manifold points, clip the array. This is considered and error but
			// is important for avoiding OOM or massive slowdowns when something goes wrong
			const int32 MaxManifoldPoints = Chaos_Collision_MaxManifoldPoints;
			if ((MaxManifoldPoints >= 0) && (NumContacts > MaxManifoldPoints))
			{
				NumContacts = MaxManifoldPoints;
				LogOneShotManifoldError(MaxManifoldPoints, ContactPoints);
			}
			
			ManifoldPoints.Reserve(NumContacts);

			for (int32 ContactIndex = 0; ContactIndex < NumContacts; ++ContactIndex)
			{
				const FContactPoint& ContactPoint = ContactPoints[ContactIndex];
				if (ContactPoint.Phi < CullDistance)
				{
					int32 ManifoldPointIndex = AddManifoldPoint(ContactPoint);
					if (ContactPoint.Phi < MinPhi)
					{
						ClosestManifoldPointIndex = ManifoldPointIndex;
						MinPhi = ContactPoint.Phi;
					}
				}
			}

			// If we have a new face we may need to change the material
			// @todo(chaos): support per-manifold point materials?
			if (NumManifoldPoints() > 0)
			{
				if (ManifoldPoints[0].ContactPoint.FaceIndex != Material.FaceIndex)
				{
					Material.FaceIndex = ManifoldPoints[0].ContactPoint.FaceIndex;
					Flags.bMaterialSet = false;
				}
			}

		}

		// Fix the contact points when we move the particle after collision detection so that
		// the contact points on each body are aligned along the normal (required for static friction tracking)
		CHAOS_API void CorrectManifoldPoints();

		CHAOS_API void UpdateManifoldContacts();

		// Particle-relative transform of each collision shape in the constraint
		const FRigidTransform3& GetShapeRelativeTransform0() const { return ImplicitTransform[0]; }
		const FRigidTransform3& GetShapeRelativeTransform1() const { return ImplicitTransform[1]; }
		const FRigidTransform3& GetShapeRelativeTransform(const int32 ParticleIndex) const { check((ParticleIndex >= 0) && (ParticleIndex < 2)); return ImplicitTransform[ParticleIndex]; }

		const FRigidTransform3& GetShapeWorldTransform0() const { return ShapeWorldTransforms[0]; }
		const FRigidTransform3& GetShapeWorldTransform1() const { return ShapeWorldTransforms[1]; }
		const FRigidTransform3& GetShapeWorldTransform(const int32 ParticleIndex) const { check((ParticleIndex >= 0) && (ParticleIndex < 2)); return ShapeWorldTransforms[ParticleIndex]; }

		void SetShapeWorldTransforms(const FRigidTransform3& InShapeWorldTransform0, const FRigidTransform3& InShapeWorldTransform1)
		{
			ShapeWorldTransforms[0] = InShapeWorldTransform0;
			ShapeWorldTransforms[1] = InShapeWorldTransform1;
		}

		// Set the transforms when we last ran collision detection. This also sets the bCanRestoreManifold flag which
		// allows the use of TryRestoreManifold on the next tick.
		void SetLastShapeWorldTransforms(const FRigidTransform3& InShapeWorldTransform0, const FRigidTransform3& InShapeWorldTransform1)
		{
			LastShapeWorldPositionDelta = InShapeWorldTransform0.GetTranslation() - InShapeWorldTransform1.GetTranslation();
			LastShapeWorldRotationDelta = InShapeWorldTransform0.GetRotation().Inverse() * InShapeWorldTransform1.GetRotation();

			// NOTE: BoundsTestFlags.bEnableManifoldUpdate is false if the shape pair does not support manifold reuse
			Flags.bCanRestoreManifold = BoundsTestFlags.bEnableManifoldUpdate;
		}

		bool GetCanRestoreManifold() const { return Flags.bCanRestoreManifold; }
		CHAOS_API bool TryRestoreManifold();
		CHAOS_API void ResetActiveManifoldContacts();
		CHAOS_API bool TryAddManifoldContact(const FContactPoint& ContactPoint);
		CHAOS_API bool TryInsertManifoldContact(const FContactPoint& ContactPoint);

		// The GJK warm-start data. This is updated directly in the narrow phase
		FGJKSimplexData& GetGJKWarmStartData() { return GJKWarmStartData; }

		const FSolverBody* GetSolverBody0() const { return SolverBodies[0]; }
		const FSolverBody* GetSolverBody1() const { return SolverBodies[1]; }

		void SetSolverBodies(const FSolverBody* InSolverBody0, const FSolverBody* InSolverBody1)
		{
			SolverBodies[0] = InSolverBody0;
			SolverBodies[1] = InSolverBody1;
		}

		/**
		 * @brief Whether this constraint was fully restored from a previous tick, and the manifold should be reused as-is
		*/
		bool WasManifoldRestored() const { return Flags.bWasManifoldRestored; }

		/**
		 * Determine the constraint direction based on Normal and Phi.
		 * This function assumes that the constraint is update-to-date.
		 */
		CHAOS_API ECollisionConstraintDirection GetConstraintDirection(const FReal Dt) const;

		/**
		 * @brief Clear the saved manifold points. This effectively resets friction anchors.
		*/
		void ResetSavedManifoldPoints()
		{
			SavedManifoldPoints.Reset();
		}

		int32 NumSavedManifoldPoints() const
		{
			return SavedManifoldPoints.Num();
		}

		const FSavedManifoldPoint& GetSavedManifoldPoint(const int32 PointIndex) const
		{
			return SavedManifoldPoints[PointIndex];
		}

		/**
		 * @brief Time of impact from CCD sweep test if CCD is activate.Otherwise undefined.
		*/
		FReal GetCCDTimeOfImpact() const { return CCDTimeOfImpact; }

		/**
		 * @brief Set the CCD TOI from the collision detection sweep
		*/
		void SetCCDTimeOfImpact(const FReal TOI) { check(TOI <= FReal(MaxTOI)); CCDTimeOfImpact = FRealSingle(TOI); }

		/**
		 * Initialize the CCD TOI to the highest possible value
		 */
		void ResetCCDTimeOfImpact() { CCDTimeOfImpact = MaxTOI; }

		/**
		 * \brief Store the results of CCD contact resolution, if active
		 */
		void SetCCDResults(const FVec3& InNetImpulse)
		{
			AccumulatedImpulse += FVec3f(InNetImpulse);
		}

		FORCEINLINE void ResetSolverResults()
		{
			// NOTE: does not initalize any manifold point data. All properties will be written to in SetSolverResults
			//SavedManifoldPoints.SetNum(ManifoldPoints.Num());
			SavedManifoldPoints.Reset(ManifoldPoints.Num());
			ManifoldPointResults.SetNum(ManifoldPoints.Num());
			MinInitialPhi = 0;
		}

		/**
		 * @brief Store the data from the solver that is retained between ticks for the specified manifold point or used by dependent systems (plasticity, breaking, etc.)
		*/
		FORCEINLINE void SetSolverResults(
			const int32 ManifoldPointIndex, 
			const FVec3f& NetPushOut, 
			const FVec3f& NetImpulse, 
			const FRealSingle StaticFrictionRatio,
			const FRealSingle Dt)
		{
			const FManifoldPoint& ManifoldPoint = ManifoldPoints[ManifoldPointIndex];
			FManifoldPointResult& ManifoldPointResult = ManifoldPointResults[ManifoldPointIndex];

			// Save contact data for friction
			bool bInsideStaticFrictionCone = false;
			if (StaticFrictionRatio >= FRealSingle(1.0f - UE_KINDA_SMALL_NUMBER))
			{
				// StaticFrictionRatio ~= 1: Static friction held - we keep the same contacts points as-is for use next frame
				SavedManifoldPoints.Emplace(
					ManifoldPoint.ShapeAnchorPoints[0],
					ManifoldPoint.ShapeAnchorPoints[1],
					ManifoldPoint.InitialPhi
				);
				bInsideStaticFrictionCone = true;
			}
			else if (StaticFrictionRatio < FRealSingle(UE_KINDA_SMALL_NUMBER))
			{
				// StaticFrictionRatio ~= 0: No friction (or no contact/impulse) - discard the friction anchors
				// If we have a lot of manifold points, we don't store the previous position for these contacts because we assume
				// that there will be others that actually applied an impulse and will fall into the other two branches.
				// This is an optimization when we have to match the new contacts with the saved ones (see AssignSavedManifoldPoints)
				const int32 SmallNumManifoldPoints = 8;
				if (ManifoldPoints.Num() < SmallNumManifoldPoints)
				{
					SavedManifoldPoints.Emplace(
						ManifoldPoint.ContactPoint.ShapeContactPoints[0],
						ManifoldPoint.ContactPoint.ShapeContactPoints[1],
						ManifoldPoint.InitialPhi
					);
				}
			}
			else
			{
				// 0 < StaticFrictionRatio < 1: We exceeded the friction cone. Slide the friction anchor 
				// toward the last-detected contact position so that it sits at the edge of the friction cone.
				SavedManifoldPoints.Emplace(
					FVec3f::Lerp(ManifoldPoint.ContactPoint.ShapeContactPoints[0], ManifoldPoint.ShapeAnchorPoints[0], StaticFrictionRatio),
					FVec3f::Lerp(ManifoldPoint.ContactPoint.ShapeContactPoints[1], ManifoldPoint.ShapeAnchorPoints[1], StaticFrictionRatio),
					ManifoldPoint.InitialPhi
				);
			}

			AccumulatedImpulse += NetImpulse + (NetPushOut / Dt);

			ManifoldPointResult.NetPushOut = NetPushOut;
			ManifoldPointResult.NetImpulse = NetImpulse;
			ManifoldPointResult.bIsValid = true;
			ManifoldPointResult.bInsideStaticFrictionCone = bInsideStaticFrictionCone;

			MinInitialPhi = FMath::Min(MinInitialPhi, ManifoldPoint.InitialPhi);
		}

		/**
		 *	A key used to uniquely identify the constraint (it is based on the two particle IDs)
		 */
		Private::FCollisionParticlePairKey GetParticlePairKey() const
		{
			return Private::FCollisionParticlePairKey(GetParticle0(), GetParticle1());
		}

		/**
		 * A key that uniquely identifies a collision constraint and provides a good sort order for the solver
		 */
		Private::FCollisionSortKey GetCollisionSortKey() const
		{
			return CollisionSortKey;
		}

		void SetCollisionSortKey(const Private::FCollisionSortKey& InCollisionSortKey)
		{
			CollisionSortKey = InCollisionSortKey;
		}

		Private::FImplicitBoundsTestFlags GetBoundsTestFlags() const
		{
			return BoundsTestFlags;
		}

	public:
		const FPBDCollisionConstraintHandle* GetConstraintHandle() const { return this; }
		FPBDCollisionConstraintHandle* GetConstraintHandle() { return this; }


		UE_DEPRECATED(5.3, "Use TryRestoreManifold")
		bool UpdateAndTryRestoreManifold() { return TryRestoreManifold(); }

	protected:

		FPBDCollisionConstraint(
			FGeometryParticleHandle* Particle0,
			const FImplicitObject* Implicit0,
			const FPerShapeData* Shape0,
			const FBVHParticles* Simplicial0,
			FGeometryParticleHandle* Particle1,
			const FImplicitObject* Implicit1,
			const FPerShapeData* Shape1,
			const FBVHParticles* Simplicial1);

		// Set all the data not initialized in the constructor
		CHAOS_API void Setup(
			const ECollisionCCDType InCCDType,
			const EContactShapesType InShapesType,
			const FRigidTransform3& InImplicitTransform0,
			const FRigidTransform3& InImplicitTransform1,
			const FReal InCullDistance,
			const bool bInUseManifold);

		// Access to the data used by the container
		const FPBDCollisionConstraintContainerCookie& GetContainerCookie() const { return ContainerCookie; }
		FPBDCollisionConstraintContainerCookie& GetContainerCookie() { return ContainerCookie; }

		CHAOS_API bool AreMatchingContactPoints(const FContactPoint& A, const FContactPoint& B, FReal& OutScore) const;
		CHAOS_API int32 FindManifoldPoint(const FContactPoint& ContactPoint) const;

		int32 FindSimpleSavedManifoldPoint(const int32 ManifoldPointIndex) const;
		CHAOS_API int32 FindSavedManifoldPoint(const int32 ManifoldPointIndex, int32* InOutAllowedSavedPointIndices, int32& InOutNumAllowedSavedPoints) const;
		CHAOS_API void AssignSavedManifoldPoints();

		inline void InitManifoldPoint(const int32 ManifoldPointIndex, const FContactPointf& ContactPoint)
		{
			FManifoldPoint& ManifoldPoint = ManifoldPoints[ManifoldPointIndex];
			ManifoldPoint.ContactPoint = ContactPoint;
			ManifoldPoint.ShapeAnchorPoints[0] = ContactPoint.ShapeContactPoints[0];
			ManifoldPoint.ShapeAnchorPoints[1] = ContactPoint.ShapeContactPoints[1];
			ManifoldPoint.InitialShapeContactPoints[0] = ContactPoint.ShapeContactPoints[0];
			ManifoldPoint.InitialShapeContactPoints[1] = ContactPoint.ShapeContactPoints[1];
			ManifoldPoint.TargetPhi = FRealSingle(0);
			ManifoldPoint.InitialPhi = FRealSingle(0);
			ManifoldPoint.Flags.Reset();
		}

		inline int32 AddManifoldPoint(const FContactPointf& ContactPoint)
		{
			int32 ManifoldPointIndex = ManifoldPoints.AddUninitialized();	// Note: no initialization (see InitManifoldPoint)
			InitManifoldPoint(ManifoldPointIndex, ContactPoint);
			return ManifoldPointIndex;
		}

		// Update the store Phi for the manifold point based on current world-space shape transforms
		// @todo(chaos): Only intended for use by the legacy solvers - remove it
		CHAOS_API void UpdateManifoldPointPhi(const int32 ManifoldPointIndex);

		CHAOS_API void InitMarginsAndTolerances(const EImplicitObjectType ImplicitType0, const EImplicitObjectType ImplicitType1, const FRealSingle Margin0, const FRealSingle Margin1);

		CHAOS_API void InitCCDThreshold();

		CHAOS_API void UpdateMaterialPropertiesImpl();

		CHAOS_API void UpdateMassScales();

	private:
		CHAOS_API FRealSingle CalculateSavedManifoldPointDistanceSq(const FSavedManifoldPoint& SavedManifoldPoint, const FManifoldPoint& ManifoldPoint) const;

		CHAOS_API void LogOneShotManifoldError(const int32 MaxManifoldPoints, const TArrayView<const FContactPoint>& ContactPoints);

		union FFlags
		{
			FFlags() : Bits(0) {}
			struct
			{
				uint16 bIsCurrent : 1;					// Was this constraint activated this tick and therefore in the current tick's active list (note: it may subsequently be disabled for various reasons)
				uint16 bDisabled : 1;					// Is this contact disabled (by the user or because cull distance is exceeded)
				uint16 bUseManifold : 1;				// Should we use contact manifolds or single points (faster but poor behaviour)
				uint16 bUseIncrementalManifold : 1;		// Do we need to run incremental collision detection (only LavelSets now)
				uint16 bCanRestoreManifold : 1;			// Can we try to restore the manifold this frame (set following narrowphase, cleared when reset for some reason)
				uint16 bWasManifoldRestored : 1;		// Did we restore the manifold this frame
				uint16 bIsQuadratic0 : 1;				// Is the first shape a sphere or capsule
				uint16 bIsQuadratic1 : 1;				// Is the second shape a sphere or capsule
				uint16 bIsProbe : 1;					// Is this constraint currently a probe
				uint16 bCCDEnabled : 1;					// Is CCD enabled for the current tick
				uint16 bCCDSweepEnabled: 1;				// If this is a CCD constraint, do we want to enable the sweep/rewind phase?
				uint16 bModifierApplied : 1;			// Was a constraint modifier applied this tick
				uint16 bMaterialSet : 1;				// Has the material been set (or does it need to be reset)
				uint16 bInitialContact : 1;				// Is this contact considered an initial contact
				uint16 bIsOneWayInteraction : 1;		// Does one of the bodies have the one-way interaction bit set?
				uint16 bUsePerContactInitialPhi : 1;	// Whether we track initial overlap per-contact or shared across all contacts in the manifold
			};
			uint16 Bits;
		};
		static_assert(sizeof(FFlags) == 2, "Unexpected size for FPBDCollisionConstraint::FFLags");

		// The margins to use during collision detection. We don't always use the margins on the shapes directly.
		// E.g., we use the smallest non-zero margin for 2 convex shapes. See InitMarginsAndTolerances
		FRealSingle CollisionMargins[2];

		// Local-space transforms of the shape (relative to particle)
		FRigidTransform3 ImplicitTransform[2];
		
		FGeometryParticleHandle* Particle[2];
		const FImplicitObject* Implicit[2];
		const FShapeInstance* Shape[2];
		const FBVHParticles* Simplicial[2];

	public:
		FVec3f AccumulatedImpulse;					// @todo(chaos): we need to accumulate angular impulse separately

	private:
		// The separation distance at which we don't track contacts
		FRealSingle CullDistance;

		// Relative movement on the last tick: (V0 - V1).Dt.
		FVec3f RelativeMovement;

		FPBDCollisionConstraintContainerCookie ContainerCookie;
		Private::FCollisionSortKey CollisionSortKey;

		// The collision tolerance is used to determine whether a new contact matches an old on. It is derived from the
		// margins of the two shapes, as well as their types
		FRealSingle CollisionTolerance;

		// The index into ManifoldPoints of the point with the lowest Phi
		int32 ClosestManifoldPointIndex;

		// The shape transforms at the current particle transforms
		FRigidTransform3 ShapeWorldTransforms[2];

		// Used by manifold point injection to see how many points were in the manifold before TryRestoreManifold
		int32 ExpectedNumManifoldPoints;

		// Relative transform the last time we ran the narrow phase
		// Used to detect when the bodies have moved too far to reues the manifold
		FVec3f LastShapeWorldPositionDelta;
		FRotation3f LastShapeWorldRotationDelta;

		// Simplex data from the last call to GJK, used to warm-start GJK
		FGJKSimplexData GJKWarmStartData;

		FPBDCollisionConstraintMaterial Material;
		FRealSingle Stiffness;

		FFlags Flags;
		Private::FImplicitBoundsTestFlags BoundsTestFlags;
		EContactShapesType ShapesType;

		template<typename T>
		using TManifoldPointArray = TArray<T, TInlineAllocator<4>>;

		TManifoldPointArray<FSavedManifoldPoint> SavedManifoldPoints;
		TManifoldPointArray<FManifoldPoint> ManifoldPoints;
		TManifoldPointArray<FManifoldPointResult> ManifoldPointResults;

		// The lowest InitialPhi value of all saved manifold points. This is used when we add new points to
		// the manifold but are still in the depenetrating phase to limit the initial overlap. 
		// We don't want the new point to cause a pop but also don't want all new contacts to be initial overlaps
		// when we were already separated at some point in the past.
		FRealSingle MinInitialPhi;

		// If we have initial overlap, the depenetration speed
		FRealSingle InitialOverlapDepenetrationVelocity;

		// Value in range [0,1] used to interpolate P between [X,P] that we will rollback to when solving at time of impact.
		FRealSingle CCDTimeOfImpact;

		// @todo(chaos): These are only needed here to support incremental collision detection for LevelSet. Fix this.
		const FSolverBody* SolverBodies[2];

		// The penetration at which CCD contacts get processed following a CCD sweep test.
		// NOTE: also see GeometryParticle CCDAxisThreshold, which is used to determine when to enable sweeping during collision detection.
		// Calculated from particle pair properties when constraint is created.
		FRealSingle CCDEnablePenetration;

		// The penetration we leave behind when rolling back to a CCD time of impact. Should be less than or equal to CCDEnablePenetration.
		// Calculated from particle pair properties when constraint is created.
		FRealSingle CCDTargetPenetration;

		friend class ::FChaosVDDataWrapperUtils;
	};


	UE_DEPRECATED(4.27, "FCollisionConstraintBase has been removed and folded into FPBDCollisionConstraint. Use FPBDCollisionConstraint")
	typedef FPBDCollisionConstraint FCollisionConstraintBase;

	UE_DEPRECATED(4.27, "FRigidBodySweptPointContactConstraint has been removed and folded into FPBDCollisionConstraint. Use FPBDCollisionConstraint")
	typedef FPBDCollisionConstraint FRigidBodySweptPointContactConstraint;
}
