// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "Chaos/CollisionResolutionTypes.h"
#include "Chaos/ConstraintHandle.h"
#include "Chaos/PBDConstraintContainer.h"

namespace Chaos
{
	class FPBDCollisionConstraints;
	class FPBDCollisionConstraint;
	class FPBDCollisionContainerSolver;

	/**
	 * @brief Whether we should run CCD (swept collision) or not
	*/
	enum class ECollisionCCDType
	{
		// Standard contact constraint
		Disabled,

		// Swept contact constraint
		Enabled,
	};

	/**
	 * @brief The resting directionality of a contact constraint for use in constraint solver ordering
	*/
	enum ECollisionConstraintDirection
	{
		// Particle 1 is on top
		Particle0ToParticle1,

		// Particle 0 is on top
		Particle1ToParticle0,

		// Neither particle is on top
		NoRestingDependency
	};

	/**
	 * @brief A handle to a contact constraint.
	 * @note This is an intrusive handle, so you can use a contact pointer as a handle.
	 * @todo(chaos): remove this class - it can just be a "using FPBDCollisionConstraintHandle = FPBDCollisionConstraint"
	*/
	class FPBDCollisionConstraintHandle : public TIntrusiveConstraintHandle<FPBDCollisionConstraint>
	{
	public:
		using Base = TIntrusiveConstraintHandle<FPBDCollisionConstraint>;
		using FImplicitPair = TPair<const FImplicitObject*, const FImplicitObject*>;
		using FGeometryPair = TPair<const TGeometryParticleHandle<FReal, 3>*, const TGeometryParticleHandle<FReal, 3>*>;
		using FHandleKey = TPair<FImplicitPair, FGeometryPair>;

		FPBDCollisionConstraintHandle()
			: TIntrusiveConstraintHandle<FPBDCollisionConstraint>()
		{
		}

		const FPBDCollisionConstraint& GetContact() const;
		FPBDCollisionConstraint& GetContact();

		UE_DEPRECATED(4.27, "Use GetContact()")
		const FPBDCollisionConstraint& GetPointContact() const { return GetContact(); }

		UE_DEPRECATED(4.27, "Use GetContact()")
		FPBDCollisionConstraint& GetPointContact() { return GetContact(); }

		UE_DEPRECATED(4.27, "Use GetContact()")
		const FPBDCollisionConstraint& GetSweptPointContact() const { return GetContact(); }

		UE_DEPRECATED(4.27, "Use GetContact()")
		FPBDCollisionConstraint& GetSweptPointContact() { return GetContact(); }

		bool GetCCDEnabled() const;

		virtual void SetEnabled(bool InEnabled) override final;

		virtual bool IsEnabled() const override final;

		virtual bool IsProbe() const override final;

		FVec3 GetAccumulatedImpulse() const;

		// Declared final so that TPBDConstraintGraphRuleImpl::AddToGraph() does not need to hit vtable
		virtual FParticlePair GetConstrainedParticles() const override final;

		const FPBDCollisionConstraints* ConcreteContainer() const;
		FPBDCollisionConstraints* ConcreteContainer();

		static CHAOS_API const FConstraintHandleTypeID& StaticType();
	};

}
