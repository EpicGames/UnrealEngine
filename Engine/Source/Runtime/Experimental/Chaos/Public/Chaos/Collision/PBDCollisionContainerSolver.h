// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/CollisionResolutionTypes.h"
#include "Chaos/Collision/CollisionApplyType.h"
#include "Chaos/Collision/PBDCollisionSolver.h"
#include "Chaos/Collision/PBDCollisionSolverSettings.h"
#include "Chaos/Evolution/SolverConstraintContainer.h"
#include "Chaos/Framework/ScratchBuffer.h"

namespace Chaos
{
	class FPBDCollisionConstraint;
	class FPBDCollisionConstraints;
	class FSolverBody;
	class FSolverBodyContainer;

	/**
	 * The solver for a set of collision constraints. This collects all the data required to solve a set of collision
	 * constraints into a contiguous, ordered buffer.
	*/
	class FPBDCollisionContainerSolver : public FConstraintContainerSolver
	{
	public:
		UE_NONCOPYABLE(FPBDCollisionContainerSolver);

		FPBDCollisionContainerSolver(const FPBDCollisionConstraints& InConstraintContainer, const int32 InPriority);
		~FPBDCollisionContainerSolver();

		int32 NumSolvers() const { return NumCollisionSolvers; }

		virtual int32 GetNumConstraints() const override final { return CollisionConstraints.Num(); }

		//
		// IslandGroup API
		//
		virtual void AddConstraints() override final;
		virtual void AddConstraints(const TArrayView<Private::FPBDIslandConstraint*>& ConstraintHandles) override final;
		virtual void AddBodies(FSolverBodyContainer& SolverBodyContainer) override final;
		virtual void GatherInput(const FReal Dt) override final;
		virtual void GatherInput(const FReal Dt, const int32 BeginIndex, const int32 EndIndex) override final;
		virtual void ScatterOutput(const FReal Dt) override final;
		virtual void ScatterOutput(const FReal Dt, const int32 BeginIndex, const int32 EndIndex) override final;
		virtual void ApplyPositionConstraints(const FReal Dt, const int32 It, const int32 NumIts) override final;
		virtual void ApplyPositionConstraints(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex) override final;
		virtual void ApplyVelocityConstraints(const FReal Dt, const int32 It, const int32 NumIts) override final;
		virtual void ApplyVelocityConstraints(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex) override final;
		virtual void ApplyProjectionConstraints(const FReal Dt, const int32 It, const int32 NumIts) override final;
		virtual void ApplyProjectionConstraints(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex) override final;

		// For testing
		const Private::FPBDCollisionSolver& GetConstraintSolver(const int32 ConstraintIndex) const { return GetSolver(ConstraintIndex); }

	private:
		virtual void ResetImpl(const int32 InMaxCollisions) override final;
		virtual bool SupportsPartitioning() const override final { return true; }
		virtual bool SortConstraintsByPartitionIndex(const TArray<int32>& PartitionIndices, const TArray<int32>& PrefixSum) override final;
		virtual void GetAllConstrainedParticles(TArray<const FGeometryParticleHandle*>& ParticleHandles) const override final;

		FPBDCollisionConstraint* GetConstraint(const int32 Index) { return CollisionConstraints[Index]; }
		const FPBDCollisionConstraint* GetConstraint(const int32 Index) const { return CollisionConstraints[Index]; }
		Private::FPBDCollisionSolver& GetSolver(const int32 Index) { check(Index < NumSolvers()); return CollisionSolvers[Index]; }
		const Private::FPBDCollisionSolver& GetSolver(const int32 Index) const { check(Index < NumSolvers()); return CollisionSolvers[Index]; }

		void CachePrefetchSolver(const int32 ConstraintIndex) const;
		void AddConstraint(FPBDCollisionConstraint& Constraint);
		size_t CalculateCollisionBufferSize(const size_t InTightFittingNum, const size_t InCurrentBufferNum) const;
		int32 CalculateConstraintMaxManifoldPoints(const FPBDCollisionConstraint* Constraint) const;
		void PrepareSolverBuffer();
		void UpdatePositionShockPropagation(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex, const FPBDCollisionSolverSettings& SolverSettings);
		void UpdateVelocityShockPropagation(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex, const FPBDCollisionSolverSettings& SolverSettings);
		void ApplyShockPropagation(const FSolverReal ShockPropagation, const int32 BeginIndex, const int32 EndIndex);
		void SolvePositionImpl(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex, const FPBDCollisionSolverSettings& SolverSettings);
		void SolveVelocityImpl(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex, const FPBDCollisionSolverSettings& SolverSettings);
		void UpdateCollisions(const FReal InDt, const int32 BeginIndex, const int32 EndIndex);

		// The constraints we are solving and the container to which they belong
		const FPBDCollisionConstraints& ConstraintContainer;
		TArray<FPBDCollisionConstraint*> CollisionConstraints;

		// The last shock propagation factor we applied
		// NOTE: Needs to be per constraint for parallelized processing
		TArray<FSolverReal> AppliedShockPropagation;

		// Buffer for all allocations for this tick (solvers and manifold points)
		Private::FScratchBuffer Scratch;

		// The start of the solver array in the scratch buffer
		Private::FPBDCollisionSolver* CollisionSolvers;
		int32 NumCollisionSolvers;

		// The start of the manifold points array in the scratch buffer
		Private::FPBDCollisionSolverManifoldPoint* CollisionSolverManifoldPoints;
		int32 NumCollisionSolverManifoldPoints;
		int32 MaxCollisionSolverManifoldPoints;

		// Whether we need to run incremental collision for each constraint (LevelSets only now)
		TArray<bool> bCollisionConstraintPerIterationCollisionDetection;
		bool bPerIterationCollisionDetection;

	};
}
