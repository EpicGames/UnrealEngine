// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/CollisionResolutionTypes.h"
#include "Chaos/Collision/CollisionApplyType.h"
#include "Chaos/Collision/PBDCollisionSolverJacobi.h"
#include "Chaos/Collision/PBDCollisionSolverSettings.h"
#include "Chaos/Evolution/SolverConstraintContainer.h"

namespace Chaos
{
	class FPBDCollisionConstraint;
	class FPBDCollisionConstraints;
	class FSolverBody;
	class FSolverBodyContainer;

	namespace Private
	{
		/**
		 * The solver for a set of collision constraints. This collects all the data required to solve a set of collision
		 * constraints into a contiguous, ordered buffer.
		 * 
		 * This version runs a Gauss-Seidel outer loop over manifolds, and a Jacobi loop over contacts in each manifold.
		*/
		class FPBDCollisionContainerSolverJacobi : public FConstraintContainerSolver
		{
		public:
			UE_NONCOPYABLE(FPBDCollisionContainerSolverJacobi);

			FPBDCollisionContainerSolverJacobi(const FPBDCollisionConstraints& InConstraintContainer, const int32 InPriority);
			~FPBDCollisionContainerSolverJacobi();

			int32 NumSolvers() const { return CollisionConstraints.Num(); }

			virtual int32 GetNumConstraints() const override final { return NumSolvers(); }

			//
			// IslandGroup API
			//
			virtual void AddConstraints() override final;
			virtual void AddConstraints(const TArrayView<Private::FPBDIslandConstraint*>& ConstraintHandles) override final;
			virtual void AddBodies(FSolverBodyContainer& SolverBodyContainer) override final;
			virtual void GatherInput(const FReal Dt) override final;
			virtual void GatherInput(const FReal Dt, const int32 BeginIndex, const int32 EndIndex) override final;
			virtual void ScatterOutput(const FReal Dt) override final;
			virtual void ScatterOutput(const FReal Dt, const int32 BeginIndex, const int32 EndIndex) override final;
			virtual void ApplyPositionConstraints(const FReal Dt, const int32 It, const int32 NumIts) override final;
			virtual void ApplyPositionConstraints(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex) override final;
			virtual void ApplyVelocityConstraints(const FReal Dt, const int32 It, const int32 NumIts) override final;
			virtual void ApplyVelocityConstraints(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex) override final;
			virtual void ApplyProjectionConstraints(const FReal Dt, const int32 It, const int32 NumIts) override final;
			virtual void ApplyProjectionConstraints(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex) override final;

		private:
			virtual bool SupportsPartitioning() const override final { return true; }
			virtual bool SortConstraintsByPartitionIndex(const TArray<int32>& PartitionIndices, const TArray<int32>& PrefixSum) override final;
			virtual void GetAllConstrainedParticles(TArray<const FGeometryParticleHandle*>& ParticleHandles) const override final;

			virtual void ResetImpl(const int32 InMaxCollisions) override final;
			void CachePrefetchSolver(const int32 ConstraintIndex) const;
			void AddConstraint(FPBDCollisionConstraint& Constraint);
			void UpdatePositionShockPropagation(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex, const FPBDCollisionSolverSettings& SolverSettings);
			void UpdateVelocityShockPropagation(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex, const FPBDCollisionSolverSettings& SolverSettings);
			bool SolvePositionImpl(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex, const FPBDCollisionSolverSettings& SolverSettings);
			bool SolveVelocityImpl(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex, const FPBDCollisionSolverSettings& SolverSettings);
			void UpdateCollisions(const FReal InDt, const int32 BeginIndex, const int32 EndIndex);

			const FPBDCollisionConstraints& ConstraintContainer;
			TArray<Private::FPBDCollisionSolverJacobi> CollisionSolvers;
			TArray<FPBDCollisionConstraint*> CollisionConstraints;
			TArray<bool> bCollisionConstraintPerIterationCollisionDetection;
			bool bPerIterationCollisionDetection;
		};

	}	// namespace Private
}	// namespace Chaos
