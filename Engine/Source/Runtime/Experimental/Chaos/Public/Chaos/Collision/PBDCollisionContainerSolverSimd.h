// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/CollisionResolutionTypes.h"
#include "Chaos/Collision/CollisionApplyType.h"
#include "Chaos/Collision/PBDCollisionSolverSimd.h"
#include "Chaos/Collision/PBDCollisionSolverSettings.h"
#include "Chaos/Evolution/SolverConstraintContainer.h"

namespace Chaos
{
	class FPBDCollisionConstraint;
	class FPBDCollisionConstraints;
	class FSolverBody;
	class FSolverBodyContainer;

	namespace Private
	{
		template<int TNumLanes>
		using TConstraintPtrSimd = TSimdValue<FPBDCollisionConstraint*, TNumLanes>;

		/**
		 * The solver for a set of collision constraints. This collects all the data required to solve a set of collision
		 * constraints into a contiguous, ordered buffer.
		 * 
		 * This version runs a Gauss-Seidel outer loop over manifolds, and a Jacobi loop over contacts in each manifold.
		*/
		class FPBDCollisionContainerSolverSimd : public FConstraintContainerSolver
		{
		public:
			UE_NONCOPYABLE(FPBDCollisionContainerSolverSimd);

			FPBDCollisionContainerSolverSimd(const FPBDCollisionConstraints& InConstraintContainer, const int32 InPriority);
			~FPBDCollisionContainerSolverSimd();

			virtual int32 GetNumConstraints() const override final { return NumConstraints; }

			//
			// IslandGroup API
			//
			virtual void AddConstraints() override final;
			virtual void AddConstraints(const TArrayView<Private::FPBDIslandConstraint*>& ConstraintHandles) override final;
			virtual void AddBodies(FSolverBodyContainer& SolverBodyContainer) override final;
			virtual void GatherInput(const FReal Dt) override final;
			virtual void GatherInput(const FReal Dt, const int32 BeginIndex, const int32 EndIndex) override final;
			virtual void ScatterOutput(const FReal Dt) override final;
			virtual void ScatterOutput(const FReal Dt, const int32 BeginIndex, const int32 EndIndex) override final;
			virtual void ApplyPositionConstraints(const FReal Dt, const int32 It, const int32 NumIts) override final;
			virtual void ApplyPositionConstraints(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex) override final;
			virtual void ApplyVelocityConstraints(const FReal Dt, const int32 It, const int32 NumIts) override final;
			virtual void ApplyVelocityConstraints(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex) override final;
			virtual void ApplyProjectionConstraints(const FReal Dt, const int32 It, const int32 NumIts) override final;
			virtual void ApplyProjectionConstraints(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex) override final;

			// Rows of collision constraints and solvers. 
			// We solve each row using SIMD. 
			// NumLanes is the width of a row and will be the same as the float SIMD register width (or less).
			template<int TNumLanes>
			struct FDataSimd
			{
				TSimdInt32<TNumLanes> SimdNumConstraints;
				TArray<TSolverBodyPtrPairSimd<TNumLanes>> SimdSolverBodies;
				TArray<TPBDCollisionSolverSimd<TNumLanes>> SimdSolvers;
				TArray<TPBDCollisionSolverManifoldPointsSimd<TNumLanes>> SimdManifoldPoints;
				TArray<TConstraintPtrSimd<TNumLanes>> SimdConstraints;
				FSolverBody DummySolverBody0[TNumLanes];
				FSolverBody DummySolverBody1[TNumLanes];
			};

			// Used to recover the solver for a constraint
			// @todo(chaos): this is only really needed for debug validation
			struct FConstraintSolverId
			{
				int32 SolverIndex;
				int32 LaneIndex;
			};

			// For testing
			const FConstraintSolverId& GetConstraintSolverId(const int32 ConstraintIndex) const { return ConstraintSolverIds[ConstraintIndex]; }
			const TPBDCollisionSolverSimd<4>& GetConstraintSolver(const int32 SolverIndex) const { return SimdData.SimdSolvers[SolverIndex]; }
			TArrayView<const TPBDCollisionSolverManifoldPointsSimd<4>> GetManifoldPointBuffer() const { return MakeArrayView(SimdData.SimdManifoldPoints); }

		private:
			virtual bool SupportsPartitioning() const override final { return false; }
			virtual bool SortConstraintsByPartitionIndex(const TArray<int32>& PartitionIndices, const TArray<int32>& PrefixSum) override final;
			virtual void GetAllConstrainedParticles(TArray<const FGeometryParticleHandle*>& ParticleHandles) const override final;

			virtual void ResetImpl(const int32 InMaxCollisions) override final;
			int32 GetNumLanes() const { return 4; }
			void CreateSolvers();
			void UpdatePositionShockPropagation(const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex, const FPBDCollisionSolverSettings& SolverSettings);
			void UpdateVelocityShockPropagation(const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex, const FPBDCollisionSolverSettings& SolverSettings);
			void ApplyShockPropagation(const FSolverReal ShockPropagation, const int32 BeginIndex, const int32 EndIndex);
			void UpdateCollisions(const FSolverReal InDt, const int32 BeginIndex, const int32 EndIndex);

			const FPBDCollisionConstraints& ConstraintContainer;

			FDataSimd<4> SimdData;
			TArray<FConstraintSolverId> ConstraintSolverIds;
			int32 NumConstraints;
			TArray<FSolverReal> AppliedShockPropagation;
			bool bPerIterationCollisionDetection;
		};

	}	// namespace Private
}	// namespace Chaos
