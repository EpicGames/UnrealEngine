// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/Collision/ContactPoint.h"
#include "Chaos/Defines.h"
#include "Chaos/Evolution/SolverBody.h"

namespace Chaos
{
	class FManifoldPoint;
	class FPBDCollisionConstraint;

	namespace Private
	{
		class FPBDCollisionSolver;
	}

	namespace CVars
	{
		extern bool bChaos_PBDCollisionSolver_Velocity_AveragePointEnabled;
		extern bool bChaos_PBDCollisionSolver_Velocity_FrictionEnabled;
		extern float Chaos_PBDCollisionSolver_Position_StaticFrictionStiffness;
		extern float Chaos_PBDCollisionSolver_Velocity_StaticFrictionStiffness;
		extern bool bChaos_PBDCollisionSolver_Use1DFriction;
	}

	namespace Private
	{
		/**
		 * @brief A single contact point in a FPBDCollisionSolver
		 * @note Internal Chaos class subject to API changes, renaming or removal
		*/
		class FPBDCollisionSolverManifoldPoint
		{
		public:
			// @todo(chaos): the contact point is only needed for SolveVelocityAverage - try to remove it
			FSolverVec3 RelativeContactPoints[2];		// R (World-space) Contact point relative to each particle's center of mass

			FSolverVec3 ContactNormal;					// N (World Space)
			FSolverVec3 ContactTangentU;				// U (World Space)
			FSolverVec3 ContactTangentV;				// V (World Space)

			FSolverReal ContactDeltaNormal;				// Initial separation along N
			FSolverReal ContactDeltaTangentU;			// Initial separation along U
			FSolverReal ContactDeltaTangentV;			// Initial separation along V

			FSolverReal ContactTargetVelocityNormal;	// Normal velocity target

			FSolverVec3 ContactRxNormal0;				// R0xN
			FSolverVec3 ContactRxNormal1;				// R1xN
			FSolverVec3 ContactRxTangentU0;				// R0xU
			FSolverVec3 ContactRxTangentV0;				// R0xV
			FSolverVec3 ContactRxTangentU1;				// R1xU
			FSolverVec3 ContactRxTangentV1;				// R1xV

			FSolverVec3 ContactNormalAngular0;			// InvI0.R0xN
			FSolverVec3 ContactTangentUAngular0;		// InvI0.R0xU
			FSolverVec3 ContactTangentVAngular0;		// InvI0.R0xV
			FSolverVec3 ContactNormalAngular1;			// InvI1.R1xN
			FSolverVec3 ContactTangentUAngular1;		// InvI1.R1xU
			FSolverVec3 ContactTangentVAngular1;		// InvI1.R1xV

			FSolverReal ContactMassNormal;				// 1/[NxR0.InvI0.R0xN + InvM0 + NxR1.InvI1.R1xN + InvM1]
			FSolverReal ContactMassTangentU;			// 1/[UxR0.InvI0.R0xU + InvM0 + UxR1.InvI1.R1xU + InvM1]
			FSolverReal ContactMassTangentV;			// 1/[VxR0.InvI0.R0xV + InvM0 + VxR1.InvI1.R1xV + InvM1]

			FSolverReal NetPushOutNormal;
			FSolverReal NetPushOutTangentU;
			FSolverReal NetPushOutTangentV;
			FSolverReal NetImpulseNormal;
			FSolverReal NetImpulseTangentU;
			FSolverReal NetImpulseTangentV;
			FSolverReal NetSoftPushOutNormal;

			FSolverReal SplitImpulseAlpha;

			// A measure of how much we exceeded the static friction threshold.
			// Equal to (NormalPushOut / TangentialPushOut) before clamping to the friction cone.
			// Used to move the static friction anchors to the edge of the cone in Scatter.
			FSolverReal StaticFrictionRatio;

			union FFlags
			{
				struct 
				{
					uint8 bDisablePositionSolve : 1;	// Disable the position solver step
					uint8 bDisableVelocitySolve : 1;	// Disable the velocity solver step
					uint8 bDisableFriction : 1;		// Disable friction
				};
				uint8 Bits;
			};

			FFlags Flags;

			// Verify that all FFlags data can be initialized to zero (even unused bits)
			static_assert(sizeof(FFlags) == sizeof(FFlags::Bits), "FPBDCollisionSolverManifoldPoint::FFlags has exceeded capacity. Change to uint16 or uint32.");
		};

		/**
		 * @brief
		 * @todo(chaos): Make this solver operate on a single contact point rather than all points in a manifold.
		 * This would be beneficial if we have many contacts with less than 4 points in the manifold. However this
		 * is dificult to do while we are still supporting non-manifold collisions.
		*/
		class FPBDCollisionSolver
		{
		public:
			static const int32 MaxConstrainedBodies = 2;

			// Create a solver that is initialized to safe defaults
			static FPBDCollisionSolver MakeInitialized()
			{
				FPBDCollisionSolver Solver;
				Solver.State.Init();
				return Solver;
			}

			// Create a solver with no initialization
			static FPBDCollisionSolver MakeUninitialized()
			{
				return FPBDCollisionSolver();
			}

			// NOTE: Does not initialize any properties. See MakeInitialized
			FPBDCollisionSolver() {}

			/** Reset the state of the collision solver */
			void Reset(FPBDCollisionSolverManifoldPoint* InManifoldPoints, const int32 InMaxManifoldPoints)
			{
				State.SolverBodies[0].Reset();
				State.SolverBodies[1].Reset();
				State.ManifoldPoints = InManifoldPoints;
				State.NumManifoldPoints = 0;
				State.MaxManifoldPoints = InMaxManifoldPoints;
			}

			void ResetManifold()
			{
				State.NumManifoldPoints = 0;
			}

			FSolverReal GetStaticFriction() const { return State.StaticFriction; }
			FSolverReal GetDynamicFriction() const { return State.DynamicFriction; }
			FSolverReal GetVelocityFriction() const { return State.VelocityFriction; }

			void SetFriction(const FSolverReal InStaticFriction, const FSolverReal InDynamicFriction, const FSolverReal InVelocityFriction, const FSolverReal InMinMaxFrictionPushOut)
			{
				State.StaticFriction = InStaticFriction;
				State.DynamicFriction = InDynamicFriction;
				State.VelocityFriction = InVelocityFriction;
				State.MinMaxFrictionPushout = InMinMaxFrictionPushOut;
			}

			void SetStiffness(const FSolverReal InStiffness)
			{
				State.Stiffness = InStiffness;
			}

			void SetHardContact()
			{
				State.SoftPhi = 0;
			}

			void SetSoftContact(const FSolverReal SoftPhi)
			{
				State.SoftPhi = SoftPhi;
			}

			void SetSolverBodies(FSolverBody& SolverBody0, FSolverBody& SolverBody1)
			{
				State.SolverBodies[0].SetSolverBody(SolverBody0);
				State.SolverBodies[1].SetSolverBody(SolverBody1);
			}

			int32 NumManifoldPoints() const
			{
				return State.NumManifoldPoints;
			}

			int32 MaxManifoldPoints() const
			{
				return State.MaxManifoldPoints;
			}

			int32 AddManifoldPoint()
			{
				if (State.NumManifoldPoints < State.MaxManifoldPoints)
				{
					return State.NumManifoldPoints++;
				}
				return INDEX_NONE;
			}

			const FPBDCollisionSolverManifoldPoint& GetManifoldPoint(const int32 ManifoldPointIndex) const
			{
				check(State.ManifoldPoints != nullptr);
				check(ManifoldPointIndex < NumManifoldPoints());

				return State.ManifoldPoints[ManifoldPointIndex];
			}

			/**
			 * Set up a manifold point (except mass properties. @see UpdateMass())
			*/
			void InitManifoldPoint(
				const int32 PointIndex,
				const FSolverReal Dt,
				const FSolverVec3& InRelativeContactPosition0,
				const FSolverVec3& InRelativeContactPosition1,
				const FSolverVec3& InWorldContactNormal,
				const FSolverVec3& InWorldContactTangentU,
				const FSolverVec3& InWorldContactTangentV,
				const FSolverReal InWorldContactDeltaNormal,
				const FSolverReal InWorldContactDeltaTangentU,
				const FSolverReal InWorldContactDeltaTangentV,
				const FSolverReal InWorldContactVelocityTargetNormal,
				const FSolverReal InSplitImpulseAlpha = 0.0f,
				const bool bDisablePositionSolve = false,
				const bool bDisableVelocitySolve = false,
				const bool bDisableFriction = false);
		
			/**
			 * Call once all manifold points have been initialized. Calculate mass properties etc.
			 */
			void FinalizeManifold();

			/**
			 * @brief Get the first solver body
			 * NOTE: This will not include any shock propagation mass scaling
			*/
			FConstraintSolverBody& SolverBody0() { return State.SolverBodies[0]; }
			const FConstraintSolverBody& SolverBody0() const { return State.SolverBodies[0]; }


			/**
			 * @brief Get the second solver body
			 * NOTE: This will not include any shock propagation mass scaling
			*/
			FConstraintSolverBody& SolverBody1() { return State.SolverBodies[1]; }
			const FConstraintSolverBody& SolverBody1() const { return State.SolverBodies[1]; }


			/**
			 * @brief Calculate and apply the position correction for this iteration
			 * @return true if we need to run more iterations, false if we did not apply any correction
			*/
			void SolvePositionNoFriction(
				const FSolverReal Dt,
				const FSolverReal MaxPushOut);

			void SolvePositionWithFriction(
				const FSolverReal Dt,
				const FSolverReal MaxPushOut);

			/**
			 * @brief Calculate and apply the velocity correction for this iteration
			 * @return true if we need to run more iterations, false if we did not apply any correction
			*/
			void SolveVelocity(
				const FSolverReal Dt,
				const bool bApplyDynamicFriction);

			void UpdateMass();

			void UpdateMassNormal();

		private:
			// Get the mass properties for the bodies if they are dynamic
			FORCEINLINE_DEBUGGABLE void GetDynamicMassProperties(
				FSolverReal& OutInvM0,
				FSolverMatrix33& OutInvI0,
				FSolverReal& OutInvM1,
				FSolverMatrix33& OutInvI1)
			{
				OutInvM0 = 0;
				OutInvM1 = 0;

				FConstraintSolverBody& Body0 = SolverBody0();
				if (Body0.IsDynamic())
				{
					OutInvM0 = Body0.InvM();
					OutInvI0 = Body0.InvI();
				}

				FConstraintSolverBody& Body1 = SolverBody1();
				if (Body1.IsDynamic())
				{
					OutInvM1 = Body1.InvM();
					OutInvI1 = Body1.InvI();
				}
			}

			bool IsDynamic(const int32 BodyIndex) const
			{
				return (State.InvMs[BodyIndex] > FSolverBody::ZeroMassThreshold());
			}

			void CalculateContactPositionCorrectionNormal(
				const FPBDCollisionSolverManifoldPoint& ManifoldPoint,
				const FSolverReal MaxPushOut,
				FSolverReal& OutContactDeltaNormal) const;

			void CalculateContactPositionErrorTangential(
				const FPBDCollisionSolverManifoldPoint& ManifoldPoint,
				FSolverReal& OutContactDeltaTangentU,
				FSolverReal& OutContactDeltaTangentV) const;

			void CalculateContactVelocityError(
				const FPBDCollisionSolverManifoldPoint& ManifoldPoint,
				const FSolverReal DynamicFriction,
				const FSolverReal Dt,
				FSolverReal& OutContactVelocityDeltaNormal,
				FSolverReal& OutContactVelocityDeltaTangent0,
				FSolverReal& OutContactVelocityDeltaTangent1) const;

			void CalculateContactVelocityErrorNormal(
				const FPBDCollisionSolverManifoldPoint& ManifoldPoint,
				FSolverReal& OutContactVelocityDeltaNormal) const;

			bool ShouldSolveVelocity(
				const FPBDCollisionSolverManifoldPoint& ManifoldPoint) const;

			void CalculatePositionCorrectionNormal(
				const FSolverReal Stiffness,
				const FSolverReal ContactDeltaNormal,
				const FSolverReal ContactMassNormal,
				const FSolverReal NetPushOutNormal,
				FSolverReal& OutPushOutNormal);

			void CalculatePositionCorrectionTangent(
				const FSolverReal Stiffness,
				const FSolverReal ContactDeltaTangent,
				const FSolverReal ContactMassTangent,
				const FSolverReal NetPushOutTangent,
				FSolverReal& OutPushOutTangent);

			void ApplyFrictionCone(
				const FSolverReal StaticFriction,
				const FSolverReal DynamicFriction,
				const FSolverReal MaxFrictionPushOut,
				FSolverReal& InOutPushOutTangentU,
				FSolverReal& InOutPushOutTangentV,
				FSolverReal& InOutNetPushOutTangentU,
				FSolverReal& InOutNetPushOutTangentV,
				FSolverReal& OutStaticFrictionRatio);

			void ApplyPositionCorrectionTangential(
				const FSolverReal Stiffness,
				const FSolverReal StaticFriction,
				const FSolverReal DynamicFriction,
				const FSolverReal MaxFrictionPushOut,
				const FSolverReal ContactDeltaTangentU,
				const FSolverReal ContactDeltaTangentV,
				FPBDCollisionSolverManifoldPoint& ManifoldPoint);

			void ApplyPositionCorrectionNormal(
				const FSolverReal Stiffness,
				const FSolverReal ContactDeltaNormal,
				FPBDCollisionSolverManifoldPoint& ManifoldPoint);

			void ApplySoftPositionCorrectionNormal(
				const FSolverReal Stiffness,
				const FSolverReal ContactDeltaNormal,
				const FSolverReal ContactVelocityNormalDt,
				FPBDCollisionSolverManifoldPoint& ManifoldPoint);

			void ApplyVelocityCorrection(
				const FSolverReal Stiffness,
				const FSolverReal Dt,
				const FSolverReal DynamicFriction,
				const FSolverReal ContactVelocityDeltaNormal,
				const FSolverReal ContactVelocityDeltaTangent0,
				const FSolverReal ContactVelocityDeltaTangent1,
				const FSolverReal MinImpulseNormal,
				FPBDCollisionSolverManifoldPoint& ManifoldPoint);

			void ApplyVelocityCorrectionNormal(
				const FSolverReal Stiffness,
				const FSolverReal ContactVelocityDeltaNormal,
				const FSolverReal MinImpulseNormal,
				FPBDCollisionSolverManifoldPoint& ManifoldPoint);

			/**
			 * @brief Run a velocity solve on the average point from all the points that received a position impulse
			 * This is used to enforce Restitution constraints without introducing rotation artefacts without
			 * adding more velocity iterations.
			 * This will only perform work if there is more than one active contact.
			*/
			void SolveVelocityAverage(const FSolverReal Dt);

			struct FState
			{
				FState()
				{
				}

				void Init()
				{
					StaticFriction = 0;
					DynamicFriction = 0;
					VelocityFriction = 0;
					Stiffness = 1;
					SolverBodies[0].Reset();
					SolverBodies[1].Reset();
					ManifoldPoints = nullptr;
					NumManifoldPoints = 0;
					MaxManifoldPoints = 0;
					SoftPhi = 0;
					MinMaxFrictionPushout = 0;
				}

				// Static Friction in the position-solve phase
				FSolverReal StaticFriction;

				// Dynamic Friction in the position-solve phase
				FSolverReal DynamicFriction;

				// Dynamic Friction in the velocity-solve phase
				FSolverReal VelocityFriction;

				// A min clamp on the max friction pushout - essentially the minimum
				// friction position impulse that is always available to this contact
				// to counteract lateral motion
				FSolverReal MinMaxFrictionPushout;

				// Solver stiffness (scales all pushout and impulses)
				FSolverReal Stiffness;

				// Soft contact penetration
				FSolverReal SoftPhi;

				// Bodies
				FConstraintSolverBody SolverBodies[MaxConstrainedBodies];
				FSolverReal InvMs[2];

				// Manifold Points
				FPBDCollisionSolverManifoldPoint* ManifoldPoints;
				int32 NumManifoldPoints;
				int32 MaxManifoldPoints;
			};

			FState State;
		};


		//////////////////////////////////////////////////////////////////////////////////////////////////
		//////////////////////////////////////////////////////////////////////////////////////////////////
		//////////////////////////////////////////////////////////////////////////////////////////////////
		//////////////////////////////////////////////////////////////////////////////////////////////////

		FORCEINLINE_DEBUGGABLE void FPBDCollisionSolver::CalculatePositionCorrectionNormal(
			const FSolverReal Stiffness,
			const FSolverReal ContactDeltaNormal,
			const FSolverReal ContactMassNormal,
			const FSolverReal NetPushOutNormal,
			FSolverReal& OutPushOutNormal)
		{
			const FSolverReal PushOutNormal = -Stiffness * ContactDeltaNormal * ContactMassNormal;

			// The total pushout so far this sub-step
			// Unilateral constraint: Net-negative impulses not allowed (negative incremental impulses are allowed as long as the net is positive)
			if ((NetPushOutNormal + PushOutNormal) > FSolverReal(0))
			{
				OutPushOutNormal = PushOutNormal;
			}
			else
			{
				OutPushOutNormal = -NetPushOutNormal;
			}
		}

		FORCEINLINE_DEBUGGABLE void FPBDCollisionSolver::CalculatePositionCorrectionTangent(
			const FSolverReal Stiffness,
			const FSolverReal ContactDeltaTangent,
			const FSolverReal ContactMassTangent,
			const FSolverReal NetPushOutTangent,
			FSolverReal& OutPushOutTangent)
		{
			// Bilateral constraint - negative values allowed (unlike the normal correction)
			OutPushOutTangent = -Stiffness * ContactMassTangent * ContactDeltaTangent;
		}

		FORCEINLINE_DEBUGGABLE void FPBDCollisionSolver::ApplyFrictionCone(
			const FSolverReal StaticFriction,
			const FSolverReal DynamicFriction,
			const FSolverReal MaxFrictionPushOut,
			FSolverReal& InOutPushOutTangentU,
			FSolverReal& InOutPushOutTangentV,
			FSolverReal& InOutNetPushOutTangentU,
			FSolverReal& InOutNetPushOutTangentV,
			FSolverReal& OutStaticFrictionRatio)
		{
			FSolverReal ClampedPushOutTangentU = InOutPushOutTangentU;
			FSolverReal ClampedPushOutTangentV = InOutPushOutTangentV;
			FSolverReal ClampedNetPushOutTangentU = InOutNetPushOutTangentU + InOutPushOutTangentU;
			FSolverReal ClampedNetPushOutTangentV = InOutNetPushOutTangentV + InOutPushOutTangentV;
			FSolverReal ClampedStaticFrictionRatio = FSolverReal(1);

			if (MaxFrictionPushOut < FSolverReal(UE_KINDA_SMALL_NUMBER))
			{
				ClampedPushOutTangentU = -InOutNetPushOutTangentU;
				ClampedPushOutTangentV = -InOutNetPushOutTangentV;
				ClampedNetPushOutTangentU = FSolverReal(0);
				ClampedNetPushOutTangentV = FSolverReal(0);
				ClampedStaticFrictionRatio = FSolverReal(0);
			}
			else
			{
				// If we exceed the static friction cone, clip to the dynamic friction cone
				const FSolverReal MaxStaticPushOutTangentSq = FMath::Square(StaticFriction * MaxFrictionPushOut);
				const FSolverReal NetPushOutTangentSq = FMath::Square(ClampedNetPushOutTangentU) + FMath::Square(ClampedNetPushOutTangentV);
				if (NetPushOutTangentSq > MaxStaticPushOutTangentSq)
				{
					const FSolverReal MaxDynamicPushOutTangent = DynamicFriction * MaxFrictionPushOut;
					const FSolverReal FrictionMultiplier = MaxDynamicPushOutTangent * FMath::InvSqrt(NetPushOutTangentSq);
					ClampedNetPushOutTangentU = FrictionMultiplier * ClampedNetPushOutTangentU;
					ClampedNetPushOutTangentV = FrictionMultiplier * ClampedNetPushOutTangentV;
					ClampedPushOutTangentU = ClampedNetPushOutTangentU - InOutNetPushOutTangentU;
					ClampedPushOutTangentV = ClampedNetPushOutTangentV - InOutNetPushOutTangentV;
					ClampedStaticFrictionRatio = FrictionMultiplier;
				}
			}

			InOutPushOutTangentU = ClampedPushOutTangentU;
			InOutPushOutTangentV = ClampedPushOutTangentV;
			InOutNetPushOutTangentU = ClampedNetPushOutTangentU;
			InOutNetPushOutTangentV = ClampedNetPushOutTangentV;
			OutStaticFrictionRatio = ClampedStaticFrictionRatio;
		}

		FORCEINLINE_DEBUGGABLE void FPBDCollisionSolver::ApplyPositionCorrectionTangential(
			const FSolverReal Stiffness,
			const FSolverReal StaticFriction,
			const FSolverReal DynamicFriction,
			const FSolverReal MaxFrictionPushOut,
			const FSolverReal ContactDeltaTangentU,
			const FSolverReal ContactDeltaTangentV,
			FPBDCollisionSolverManifoldPoint& ManifoldPoint)
		{
			FSolverReal PushOutTangentU = FSolverReal(0);
			FSolverReal PushOutTangentV = FSolverReal(0);

			CalculatePositionCorrectionTangent(
				Stiffness,
				ContactDeltaTangentU,
				ManifoldPoint.ContactMassTangentU,
				ManifoldPoint.NetPushOutTangentU,
				PushOutTangentU);					// Out

			CalculatePositionCorrectionTangent(
				Stiffness,
				ContactDeltaTangentV,
				ManifoldPoint.ContactMassTangentV,
				ManifoldPoint.NetPushOutTangentV,
				PushOutTangentV);					// Out

			// NOTE: This function modifies NetPushOutTangentU and V
			ApplyFrictionCone(
				StaticFriction,
				DynamicFriction,
				MaxFrictionPushOut,
				PushOutTangentU,									// InOut
				PushOutTangentV,									// InOut
				ManifoldPoint.NetPushOutTangentU,					// InOut
				ManifoldPoint.NetPushOutTangentV,					// InOut
				ManifoldPoint.StaticFrictionRatio);					// Out

			// Update the particle state based on the pushout
			const FVec3 PushOut = PushOutTangentU * ManifoldPoint.ContactTangentU + PushOutTangentV * ManifoldPoint.ContactTangentV;
			if (IsDynamic(0))
			{
				const FSolverVec3 DX0 = State.InvMs[0] * PushOut;
				const FSolverVec3 DR0 = ManifoldPoint.ContactTangentUAngular0 * PushOutTangentU + ManifoldPoint.ContactTangentVAngular0 * PushOutTangentV;
				FConstraintSolverBody& Body0 = SolverBody0();

				// NOTE: No split impulse mode for friction
				Body0.ApplyPositionDelta(DX0);
				Body0.ApplyRotationDelta(DR0);
			}
			if (IsDynamic(1))
			{
				const FSolverVec3 DX1 = -State.InvMs[1] * PushOut;
				const FSolverVec3 DR1 = ManifoldPoint.ContactTangentUAngular1 * -PushOutTangentU + ManifoldPoint.ContactTangentVAngular1 * -PushOutTangentV;
				FConstraintSolverBody& Body1 = SolverBody1();

				// NOTE: No split impulse mode for friction
				Body1.ApplyPositionDelta(DX1);
				Body1.ApplyRotationDelta(DR1);
			}
		}

		FORCEINLINE_DEBUGGABLE void FPBDCollisionSolver::ApplyPositionCorrectionNormal(
			const FSolverReal Stiffness,
			const FSolverReal ContactDeltaNormal,
			FPBDCollisionSolverManifoldPoint& ManifoldPoint)
		{
			FSolverReal PushOutNormal = FSolverReal(0);

			// How much of this position correction should be momentumless?
			// In SplitImpulse mode, only residual overlap corrections apply momentum.
			FSolverReal SplitAlpha = ManifoldPoint.SplitImpulseAlpha;
			FSolverReal SplitAlphaInv = FSolverReal(1) - ManifoldPoint.SplitImpulseAlpha;

			CalculatePositionCorrectionNormal(
				Stiffness,
				ContactDeltaNormal,
				ManifoldPoint.ContactMassNormal,
				ManifoldPoint.NetPushOutNormal,
				PushOutNormal);						// Out


			ManifoldPoint.NetPushOutNormal += SplitAlphaInv * PushOutNormal;

			// Update the particle state based on the pushout
			if (IsDynamic(0))
			{
				const FSolverVec3 DX0 = (State.InvMs[0] * PushOutNormal) * ManifoldPoint.ContactNormal;
				const FSolverVec3 DR0 = ManifoldPoint.ContactNormalAngular0 * PushOutNormal;
				FConstraintSolverBody& Body0 = SolverBody0();
				if (SplitAlpha > 0)
				{
					Body0.ApplyPositionCorrectionDelta(SplitAlpha * DX0);
					Body0.ApplyRotationCorrectionDelta(SplitAlpha * DR0);
				}

				if (SplitAlphaInv > 0)
				{
					Body0.ApplyPositionDelta(SplitAlphaInv * DX0);
					Body0.ApplyRotationDelta(SplitAlphaInv * DR0);
				}
			}
			if (IsDynamic(1))
			{
				const FSolverVec3 DX1 = (State.InvMs[1] * -PushOutNormal) * ManifoldPoint.ContactNormal;
				const FSolverVec3 DR1 = ManifoldPoint.ContactNormalAngular1 * -PushOutNormal;
				FConstraintSolverBody& Body1 = SolverBody1();
				if (SplitAlpha > 0)
				{
					Body1.ApplyPositionCorrectionDelta(SplitAlpha * DX1);
					Body1.ApplyRotationCorrectionDelta(SplitAlpha * DR1);
				}

				if (SplitAlphaInv > 0)
				{
					Body1.ApplyPositionDelta(SplitAlphaInv * DX1);
					Body1.ApplyRotationDelta(SplitAlphaInv * DR1);
				}
			}
		}

		FORCEINLINE_DEBUGGABLE void FPBDCollisionSolver::ApplySoftPositionCorrectionNormal(
			const FSolverReal Stiffness,
			const FSolverReal ContactDeltaNormal,
			const FSolverReal ContactVelocityNormalDt,
			FPBDCollisionSolverManifoldPoint& ManifoldPoint)
		{
			// TODO: Apply soft pushout correction
		}

		FORCEINLINE_DEBUGGABLE void FPBDCollisionSolver::ApplyVelocityCorrection(
			const FSolverReal Stiffness,
			const FSolverReal Dt,
			const FSolverReal DynamicFriction,
			const FSolverReal ContactVelocityDeltaNormal,
			const FSolverReal ContactVelocityDeltaTangent0,
			const FSolverReal ContactVelocityDeltaTangent1,
			const FSolverReal MinImpulseNormal,
			FPBDCollisionSolverManifoldPoint& ManifoldPoint)
		{
			FSolverReal ImpulseNormal = -Stiffness * ManifoldPoint.ContactMassNormal * ContactVelocityDeltaNormal;

			// Clamp the total impulse to be positive along the normal. We can apply a net negative impulse, 
			// but only to correct the velocity that was added by pushout (in which case MinImpulseNormal will be negative).
			if ((ManifoldPoint.NetImpulseNormal + ImpulseNormal) < MinImpulseNormal)
			{
				// We are trying to apply a net negative impulse larger than one to counteract the effective pushout impulse
				// so clamp the net impulse to be equal to minus the pushout impulse along the normal.
				ImpulseNormal = MinImpulseNormal - ManifoldPoint.NetImpulseNormal;
			}

			ManifoldPoint.NetImpulseNormal += ImpulseNormal;

			FSolverVec3 Impulse = ImpulseNormal * ManifoldPoint.ContactNormal;

			// Clamp the tangential impulses to the friction cone
			FSolverReal ImpulseTangentU = 0;
			FSolverReal ImpulseTangentV = 0;
			if ((DynamicFriction > 0) && (Dt > 0))
			{
				const FSolverReal FrictionStiffness = Stiffness * CVars::Chaos_PBDCollisionSolver_Velocity_StaticFrictionStiffness;
				ImpulseTangentU = -FrictionStiffness * ManifoldPoint.ContactMassTangentU * ContactVelocityDeltaTangent0;
				ImpulseTangentV = -FrictionStiffness * ManifoldPoint.ContactMassTangentV * ContactVelocityDeltaTangent1;

				const FSolverReal TotalImpulseNormal
					= ManifoldPoint.NetImpulseNormal
					+ (ManifoldPoint.NetSoftPushOutNormal
					+  ManifoldPoint.NetPushOutNormal) / Dt;

				// Clamp the total tangential impulse including positional contribution to the friction cone
				const FSolverReal MinMaxFrictionImpulse = State.MinMaxFrictionPushout / Dt;
				const FSolverReal MaxImpulseTangent = DynamicFriction * FMath::Max(MinMaxFrictionImpulse, TotalImpulseNormal);
				const FSolverReal MaxImpulseTangentSq = FMath::Square(MaxImpulseTangent);
				const FSolverReal NetImpulseTangentU = ManifoldPoint.NetImpulseTangentU + ManifoldPoint.NetPushOutTangentU / Dt;
				const FSolverReal NetImpulseTangentV = ManifoldPoint.NetImpulseTangentV + ManifoldPoint.NetPushOutTangentV / Dt;
				const FSolverReal NetImpulseTangentSq = FMath::Square(NetImpulseTangentU + ImpulseTangentU) + FMath::Square(NetImpulseTangentV + ImpulseTangentV);
				if (NetImpulseTangentSq > (MaxImpulseTangentSq + UE_SMALL_NUMBER))
				{
					const FSolverReal ImpulseTangentScale = MaxImpulseTangent * FMath::InvSqrt(NetImpulseTangentSq);
					ImpulseTangentU = (NetImpulseTangentU + ImpulseTangentU) * ImpulseTangentScale - NetImpulseTangentU;
					ImpulseTangentV = (NetImpulseTangentV + ImpulseTangentV) * ImpulseTangentScale - NetImpulseTangentV;
					ManifoldPoint.StaticFrictionRatio = ImpulseTangentScale;
				}

				ManifoldPoint.NetImpulseTangentU += ImpulseTangentU;
				ManifoldPoint.NetImpulseTangentV += ImpulseTangentV;

				Impulse += ImpulseTangentU * ManifoldPoint.ContactTangentU + ImpulseTangentV * ManifoldPoint.ContactTangentV;
			}

			// Apply the velocity deltas from the impulse
			if (IsDynamic(0))
			{
				const FSolverVec3 DV0 = State.InvMs[0] * Impulse;
				const FSolverVec3 DW0 = ManifoldPoint.ContactNormalAngular0 * ImpulseNormal + ManifoldPoint.ContactTangentUAngular0 * ImpulseTangentU + ManifoldPoint.ContactTangentVAngular0 * ImpulseTangentV;
				FConstraintSolverBody& Body0 = SolverBody0();
				Body0.ApplyVelocityDelta(DV0, DW0);
			}
			if (IsDynamic(1))
			{
				const FSolverVec3 DV1 = -State.InvMs[1] * Impulse;
				const FSolverVec3 DW1 = ManifoldPoint.ContactNormalAngular1 * -ImpulseNormal + ManifoldPoint.ContactTangentUAngular1 * -ImpulseTangentU + ManifoldPoint.ContactTangentVAngular1 * -ImpulseTangentV;
				FConstraintSolverBody& Body1 = SolverBody1();
				Body1.ApplyVelocityDelta(DV1, DW1);
			}
		}

		FORCEINLINE_DEBUGGABLE void FPBDCollisionSolver::ApplyVelocityCorrectionNormal(
			const FSolverReal Stiffness,
			const FSolverReal ContactVelocityDeltaNormal,
			const FSolverReal MinImpulseNormal,
			FPBDCollisionSolverManifoldPoint& ManifoldPoint)
		{
			FSolverReal ImpulseNormal = -(Stiffness * ManifoldPoint.ContactMassNormal) * ContactVelocityDeltaNormal;

			// See comments in ApplyVelocityCorrection
			if (ManifoldPoint.NetImpulseNormal + ImpulseNormal < MinImpulseNormal)
			{
				ImpulseNormal = MinImpulseNormal - ManifoldPoint.NetImpulseNormal;
			}

			ManifoldPoint.NetImpulseNormal += ImpulseNormal;

			// Calculate the velocity deltas from the impulse
			FSolverVec3 Impulse = ImpulseNormal * ManifoldPoint.ContactNormal;
			if (IsDynamic(0))
			{
				const FSolverVec3 DV0 = State.InvMs[0] * Impulse;
				const FSolverVec3 DW0 = ManifoldPoint.ContactNormalAngular0 * ImpulseNormal;
				FConstraintSolverBody& Body0 = SolverBody0();
				Body0.ApplyVelocityDelta(DV0, DW0);
			}
			if (IsDynamic(1))
			{
				const FSolverVec3 DV1 = -State.InvMs[1] * Impulse;
				const FSolverVec3 DW1 = ManifoldPoint.ContactNormalAngular1 * -ImpulseNormal;
				FConstraintSolverBody& Body1 = SolverBody1();
				Body1.ApplyVelocityDelta(DV1, DW1);
			}
		}

		FORCEINLINE_DEBUGGABLE void FPBDCollisionSolver::UpdateMass()
		{
			const FConstraintSolverBody& Body0 = SolverBody0();
			const FConstraintSolverBody& Body1 = SolverBody1();

			FSolverReal InvM0, InvM1;
			FSolverMatrix33 InvI0, InvI1;
			GetDynamicMassProperties(InvM0, InvI0, InvM1, InvI1);

			State.InvMs[0] = InvM0;
			State.InvMs[1] = InvM1;

			for (int32 PointIndex = 0; PointIndex < NumManifoldPoints(); ++PointIndex)
			{
				FPBDCollisionSolverManifoldPoint& ManifoldPoint = State.ManifoldPoints[PointIndex];

				FSolverReal ContactMassInvNormal = FSolverReal(0);
				FSolverReal ContactMassInvTangentU = FSolverReal(0);
				FSolverReal ContactMassInvTangentV = FSolverReal(0);

				// These are not used if not initialized below so no need to clear
				//ContactNormalAngular0 = FSolverVec3(0);
				//ContactTangentUAngular0 = FSolverVec3(0);
				//ContactTangentVAngular0 = FSolverVec3(0);
				//ContactNormalAngular1 = FSolverVec3(0);
				//ContactTangentUAngular1 = FSolverVec3(0);
				//ContactTangentVAngular1 = FSolverVec3(0);

				if (IsDynamic(0))
				{
					const FSolverVec3& R0xN = ManifoldPoint.ContactRxNormal0;
					const FSolverVec3& R0xU = ManifoldPoint.ContactRxTangentU0;
					const FSolverVec3& R0xV = ManifoldPoint.ContactRxTangentV0;

					ManifoldPoint.ContactNormalAngular0 = InvI0 * R0xN;
					ManifoldPoint.ContactTangentUAngular0 = InvI0 * R0xU;
					ManifoldPoint.ContactTangentVAngular0 = InvI0 * R0xV;

					ContactMassInvNormal += FSolverVec3::DotProduct(R0xN, ManifoldPoint.ContactNormalAngular0) + InvM0;
					ContactMassInvTangentU += FSolverVec3::DotProduct(R0xU, ManifoldPoint.ContactTangentUAngular0) + InvM0;
					ContactMassInvTangentV += FSolverVec3::DotProduct(R0xV, ManifoldPoint.ContactTangentVAngular0) + InvM0;
				}
				if (IsDynamic(1))
				{
					const FSolverVec3& R1xN = ManifoldPoint.ContactRxNormal1;
					const FSolverVec3& R1xU = ManifoldPoint.ContactRxTangentU1;
					const FSolverVec3& R1xV = ManifoldPoint.ContactRxTangentV1;

					ManifoldPoint.ContactNormalAngular1 = InvI1 * R1xN;
					ManifoldPoint.ContactTangentUAngular1 = InvI1 * R1xU;
					ManifoldPoint.ContactTangentVAngular1 = InvI1 * R1xV;

					ContactMassInvNormal += FSolverVec3::DotProduct(R1xN, ManifoldPoint.ContactNormalAngular1) + InvM1;
					ContactMassInvTangentU += FSolverVec3::DotProduct(R1xU, ManifoldPoint.ContactTangentUAngular1) + InvM1;
					ContactMassInvTangentV += FSolverVec3::DotProduct(R1xV, ManifoldPoint.ContactTangentVAngular1) + InvM1;
				}

				ManifoldPoint.ContactMassNormal = (ContactMassInvNormal > FSolverReal(UE_SMALL_NUMBER)) ? FSolverReal(1) / ContactMassInvNormal : FSolverReal(0);
				ManifoldPoint.ContactMassTangentU = (ContactMassInvTangentU > FSolverReal(UE_SMALL_NUMBER)) ? FSolverReal(1) / ContactMassInvTangentU : FSolverReal(0);
				ManifoldPoint.ContactMassTangentV = (ContactMassInvTangentV > FSolverReal(UE_SMALL_NUMBER)) ? FSolverReal(1) / ContactMassInvTangentV : FSolverReal(0);
			}
		}

		FORCEINLINE_DEBUGGABLE void FPBDCollisionSolver::UpdateMassNormal()
		{
			const FConstraintSolverBody& Body0 = SolverBody0();
			const FConstraintSolverBody& Body1 = SolverBody1();

			FSolverReal InvM0, InvM1;
			FSolverMatrix33 InvI0, InvI1;
			GetDynamicMassProperties(InvM0, InvI0, InvM1, InvI1);

			State.InvMs[0] = InvM0;
			State.InvMs[1] = InvM1;

			for (int32 PointIndex = 0; PointIndex < NumManifoldPoints(); ++PointIndex)
			{
				FPBDCollisionSolverManifoldPoint& ManifoldPoint = State.ManifoldPoints[PointIndex];

				FSolverReal ContactMassInvNormal = FSolverReal(0);
				if (IsDynamic(0))
				{
					const FSolverVec3& R0xN = ManifoldPoint.ContactRxNormal0;
					ManifoldPoint.ContactNormalAngular0 = InvI0 * R0xN;
					ContactMassInvNormal += FSolverVec3::DotProduct(R0xN, ManifoldPoint.ContactNormalAngular0) + InvM0;
				}
				if (IsDynamic(1))
				{
					const FSolverVec3& R1xN = ManifoldPoint.ContactRxNormal1;
					ManifoldPoint.ContactNormalAngular1 = InvI1 * R1xN;
					ContactMassInvNormal += FSolverVec3::DotProduct(R1xN, ManifoldPoint.ContactNormalAngular1) + InvM1;
				}
				ManifoldPoint.ContactMassNormal = (ContactMassInvNormal > FSolverReal(UE_SMALL_NUMBER)) ? FSolverReal(1) / ContactMassInvNormal : FSolverReal(0);
			}
		}

		FORCEINLINE_DEBUGGABLE void FPBDCollisionSolver::CalculateContactPositionCorrectionNormal(
			const FPBDCollisionSolverManifoldPoint& ManifoldPoint,
			const FSolverReal MaxPushOut,
			FSolverReal& OutContactDeltaNormal) const
		{
			const FConstraintSolverBody& Body0 = SolverBody0();
			const FConstraintSolverBody& Body1 = SolverBody1();

			// Linear version: calculate the contact delta assuming linear motion after applying a positional impulse at the contact point. There will be an error that depends on the size of the rotation.
			FSolverReal ContactDelta = 0;
			if (IsDynamic(0) && IsDynamic(1))
			{
				ContactDelta += FSolverVec3::DotProduct(Body0.DP() - Body1.DP(), ManifoldPoint.ContactNormal);
				ContactDelta += FSolverVec3::DotProduct(Body0.DQ(), ManifoldPoint.ContactRxNormal0);
				ContactDelta -= FSolverVec3::DotProduct(Body1.DQ(), ManifoldPoint.ContactRxNormal1);
			}
			else if (IsDynamic(0))
			{
				ContactDelta += FSolverVec3::DotProduct(Body0.DQ(), ManifoldPoint.ContactRxNormal0);
				ContactDelta += FSolverVec3::DotProduct(Body0.DP(), ManifoldPoint.ContactNormal);
			}
			else if (IsDynamic(1))
			{
				ContactDelta -= FSolverVec3::DotProduct(Body1.DQ(), ManifoldPoint.ContactRxNormal1);
				ContactDelta -= FSolverVec3::DotProduct(Body1.DP(), ManifoldPoint.ContactNormal);
			}

			// NOTE: ContactDelta is negative for penetration
			// NOTE: MaxPushOut == 0 disables the pushout limits
			if ((MaxPushOut > 0) && (ContactDelta < -MaxPushOut))
			{
				ContactDelta = -MaxPushOut;
			}

			OutContactDeltaNormal = ContactDelta;
		}

		FORCEINLINE_DEBUGGABLE void FPBDCollisionSolver::CalculateContactPositionErrorTangential(
			const FPBDCollisionSolverManifoldPoint& ManifoldPoint,
			FSolverReal& OutContactDeltaTangentU,
			FSolverReal& OutContactDeltaTangentV) const
		{
			const FConstraintSolverBody& Body0 = SolverBody0();
			const FConstraintSolverBody& Body1 = SolverBody1();

			// Linear version: calculate the contact delta assuming linear motion after applying a positional impulse at the contact point. There will be an error that depends on the size of the rotation.
			FSolverReal ContactDeltaU = ManifoldPoint.ContactDeltaTangentU;
			FSolverReal ContactDeltaV = ManifoldPoint.ContactDeltaTangentV;
			if (IsDynamic(0) && IsDynamic(1))
			{
				ContactDeltaU += FSolverVec3::DotProduct(Body0.DP() - Body1.DP(), ManifoldPoint.ContactTangentU);
				ContactDeltaU += FSolverVec3::DotProduct(Body0.DQ(), ManifoldPoint.ContactRxTangentU0);
				ContactDeltaU -= FSolverVec3::DotProduct(Body1.DQ(), ManifoldPoint.ContactRxTangentU1);

				ContactDeltaV += FSolverVec3::DotProduct(Body0.DP() - Body1.DP(), ManifoldPoint.ContactTangentV);
				ContactDeltaV += FSolverVec3::DotProduct(Body0.DQ(), ManifoldPoint.ContactRxTangentV0);
				ContactDeltaV -= FSolverVec3::DotProduct(Body1.DQ(), ManifoldPoint.ContactRxTangentV1);
			}
			else if (IsDynamic(0))
			{
				ContactDeltaU += FSolverVec3::DotProduct(Body0.DP(), ManifoldPoint.ContactTangentU);
				ContactDeltaU += FSolverVec3::DotProduct(Body0.DQ(), ManifoldPoint.ContactRxTangentU0);

				ContactDeltaV += FSolverVec3::DotProduct(Body0.DP(), ManifoldPoint.ContactTangentV);
				ContactDeltaV += FSolverVec3::DotProduct(Body0.DQ(), ManifoldPoint.ContactRxTangentV0);
			}
			else if (IsDynamic(1))
			{
				ContactDeltaU -= FSolverVec3::DotProduct(Body1.DP(), ManifoldPoint.ContactTangentU);
				ContactDeltaU -= FSolverVec3::DotProduct(Body1.DQ(), ManifoldPoint.ContactRxTangentU1);

				ContactDeltaV -= FSolverVec3::DotProduct(Body1.DP(), ManifoldPoint.ContactTangentV);
				ContactDeltaV -= FSolverVec3::DotProduct(Body1.DQ(), ManifoldPoint.ContactRxTangentV1);
			}

			OutContactDeltaTangentU = ContactDeltaU;
			OutContactDeltaTangentV = ContactDeltaV;
		}

		FORCEINLINE_DEBUGGABLE void FPBDCollisionSolver::CalculateContactVelocityError(
			const FPBDCollisionSolverManifoldPoint& ManifoldPoint,
			const FSolverReal DynamicFriction,
			const FSolverReal Dt,
			FSolverReal& OutContactVelocityDeltaNormal,
			FSolverReal& OutContactVelocityDeltaTangentU,
			FSolverReal& OutContactVelocityDeltaTangentV) const
		{
			const FConstraintSolverBody& Body0 = SolverBody0();
			const FConstraintSolverBody& Body1 = SolverBody1();

			FSolverReal ContactVelN = -ManifoldPoint.ContactTargetVelocityNormal;
			FSolverReal ContactVelU = FSolverReal(0);
			FSolverReal ContactVelV = FSolverReal(0);

			// @todo(chaos): check for static or stationary kinematics?
			ContactVelN += FSolverVec3::DotProduct(Body0.V() - Body1.V(), ManifoldPoint.ContactNormal);
			ContactVelN += FSolverVec3::DotProduct(Body0.W(), ManifoldPoint.ContactRxNormal0);
			ContactVelN -= FSolverVec3::DotProduct(Body1.W(), ManifoldPoint.ContactRxNormal1);

			ContactVelU += FSolverVec3::DotProduct(Body0.V() - Body1.V(), ManifoldPoint.ContactTangentU);
			ContactVelU += FSolverVec3::DotProduct(Body0.W(), ManifoldPoint.ContactRxTangentU0);
			ContactVelU -= FSolverVec3::DotProduct(Body1.W(), ManifoldPoint.ContactRxTangentU1);

			ContactVelV += FSolverVec3::DotProduct(Body0.V() - Body1.V(), ManifoldPoint.ContactTangentV);
			ContactVelV += FSolverVec3::DotProduct(Body0.W(), ManifoldPoint.ContactRxTangentV0);
			ContactVelV -= FSolverVec3::DotProduct(Body1.W(), ManifoldPoint.ContactRxTangentV1);

			OutContactVelocityDeltaNormal = ContactVelN;
			OutContactVelocityDeltaTangentU = ContactVelU;
			OutContactVelocityDeltaTangentV = ContactVelV;
		}

		FORCEINLINE_DEBUGGABLE void FPBDCollisionSolver::CalculateContactVelocityErrorNormal(
			const FPBDCollisionSolverManifoldPoint& ManifoldPoint,
			FSolverReal& OutContactVelocityDeltaNormal) const
		{
			const FConstraintSolverBody& Body0 = SolverBody0();
			const FConstraintSolverBody& Body1 = SolverBody1();

			FSolverReal ContactVelN = -ManifoldPoint.ContactTargetVelocityNormal;
			ContactVelN += FSolverVec3::DotProduct(Body0.V() - Body1.V(), ManifoldPoint.ContactNormal);
			ContactVelN += FSolverVec3::DotProduct(Body0.W(), ManifoldPoint.ContactRxNormal0);
			ContactVelN -= FSolverVec3::DotProduct(Body1.W(), ManifoldPoint.ContactRxNormal1);

			OutContactVelocityDeltaNormal = ContactVelN;
		}

		FORCEINLINE_DEBUGGABLE bool FPBDCollisionSolver::ShouldSolveVelocity(
			const FPBDCollisionSolverManifoldPoint& ManifoldPoint) const
		{
			// We ensure positive separating velocity for close contacts even if they didn't receive a pushout
			return (ManifoldPoint.NetPushOutNormal > FSolverReal(0)) || (ManifoldPoint.NetSoftPushOutNormal > FSolverReal(0)) || (ManifoldPoint.ContactDeltaNormal < State.SoftPhi);
		}

		FORCEINLINE_DEBUGGABLE void FPBDCollisionSolver::InitManifoldPoint(
			const int32 PointIndex,
			const FSolverReal Dt,
			const FSolverVec3& InRelativeContactPosition0,
			const FSolverVec3& InRelativeContactPosition1,
			const FSolverVec3& InWorldContactNormal,
			const FSolverVec3& InWorldContactTangentU,
			const FSolverVec3& InWorldContactTangentV,
			const FSolverReal InWorldContactDeltaNormal,
			const FSolverReal InWorldContactDeltaTangentU,
			const FSolverReal InWorldContactDeltaTangentV,
			const FSolverReal InWorldContactVelocityTargetNormal,
			const FSolverReal InSplitImpulseAlpha,
			const bool bDisablePositionSolve,
			const bool bDisableVelocitySolve,
			const bool bDisableFriction
			)
		{
			check(State.ManifoldPoints != nullptr);
			check(PointIndex >= 0);
			check(PointIndex < State.NumManifoldPoints);
			check(InSplitImpulseAlpha >= 0.0f);
			check(InSplitImpulseAlpha <= 1.0f);

			FPBDCollisionSolverManifoldPoint& ManifoldPoint = State.ManifoldPoints[PointIndex];

			ManifoldPoint.RelativeContactPoints[0] = InRelativeContactPosition0;
			ManifoldPoint.RelativeContactPoints[1] = InRelativeContactPosition1;

			ManifoldPoint.ContactNormal = InWorldContactNormal;
			ManifoldPoint.ContactTangentU = InWorldContactTangentU;
			ManifoldPoint.ContactTangentV = InWorldContactTangentV;
			ManifoldPoint.ContactDeltaNormal = InWorldContactDeltaNormal;
			ManifoldPoint.ContactDeltaTangentU = InWorldContactDeltaTangentU;
			ManifoldPoint.ContactDeltaTangentV = InWorldContactDeltaTangentV;
			ManifoldPoint.ContactTargetVelocityNormal = InWorldContactVelocityTargetNormal;
			ManifoldPoint.SplitImpulseAlpha = InSplitImpulseAlpha;

			ManifoldPoint.ContactRxNormal0 = FSolverVec3::CrossProduct(InRelativeContactPosition0, InWorldContactNormal);
			ManifoldPoint.ContactRxTangentU0 = FSolverVec3::CrossProduct(InRelativeContactPosition0, InWorldContactTangentU);
			ManifoldPoint.ContactRxTangentV0 = FSolverVec3::CrossProduct(InRelativeContactPosition0, InWorldContactTangentV);
			ManifoldPoint.ContactRxNormal1 = FSolverVec3::CrossProduct(InRelativeContactPosition1, InWorldContactNormal);
			ManifoldPoint.ContactRxTangentU1 = FSolverVec3::CrossProduct(InRelativeContactPosition1, InWorldContactTangentU);
			ManifoldPoint.ContactRxTangentV1 = FSolverVec3::CrossProduct(InRelativeContactPosition1, InWorldContactTangentV);

			ManifoldPoint.NetPushOutNormal = FSolverReal(0);
			ManifoldPoint.NetPushOutTangentU = FSolverReal(0);
			ManifoldPoint.NetPushOutTangentV = FSolverReal(0);
			ManifoldPoint.NetImpulseNormal = FSolverReal(0);
			ManifoldPoint.NetImpulseTangentU = FSolverReal(0);
			ManifoldPoint.NetImpulseTangentV = FSolverReal(0);
			ManifoldPoint.NetSoftPushOutNormal = FSolverReal(0);
			ManifoldPoint.StaticFrictionRatio = FSolverReal(0);

			ManifoldPoint.Flags.Bits = 0;
			ManifoldPoint.Flags.bDisablePositionSolve = bDisablePositionSolve;
			ManifoldPoint.Flags.bDisableVelocitySolve = bDisableVelocitySolve;
			ManifoldPoint.Flags.bDisableFriction = bDisableFriction;
		}

		FORCEINLINE_DEBUGGABLE void FPBDCollisionSolver::FinalizeManifold()
		{
			UpdateMass();
		}

		FORCEINLINE_DEBUGGABLE void FPBDCollisionSolver::SolvePositionWithFriction(
			const FSolverReal Dt,
			const FSolverReal MaxPushOut)
		{
			// Apply the position correction along the normal and determine if we want to run friction on each point
			SolvePositionNoFriction(Dt, MaxPushOut);

			// Apply the tangential position correction if required
			const FSolverReal FrictionStiffness = State.Stiffness * CVars::Chaos_PBDCollisionSolver_Position_StaticFrictionStiffness;
			if (FrictionStiffness > 0)
			{
				for (int32 PointIndex = 0; PointIndex < NumManifoldPoints(); ++PointIndex)
				{
					FPBDCollisionSolverManifoldPoint& SolverManifoldPoint = State.ManifoldPoints[PointIndex];
					const bool bFrictionSolveDisabled = !!SolverManifoldPoint.Flags.bDisableFriction || !!SolverManifoldPoint.Flags.bDisablePositionSolve;
					if (bFrictionSolveDisabled)
					{
						continue;
					}

					const FSolverReal TotalPushOutNormal
						= SolverManifoldPoint.NetPushOutNormal
						+ SolverManifoldPoint.NetSoftPushOutNormal;

					const FSolverReal FrictionMaxPushOut = FMath::Max(State.MinMaxFrictionPushout, TotalPushOutNormal);

					const bool bApplyFriction = (FrictionMaxPushOut > 0) || (SolverManifoldPoint.NetPushOutTangentU != 0) || (SolverManifoldPoint.NetPushOutTangentV != 0);

					if (bApplyFriction)
					{
						FSolverReal ContactDeltaTangentU, ContactDeltaTangentV;
						CalculateContactPositionErrorTangential(SolverManifoldPoint, ContactDeltaTangentU, ContactDeltaTangentV);

						ApplyPositionCorrectionTangential(
							FrictionStiffness,
							State.StaticFriction,
							State.DynamicFriction,
							FrictionMaxPushOut,
							ContactDeltaTangentU,
							ContactDeltaTangentV,
							SolverManifoldPoint);
					}
				}
			}
		}

		FORCEINLINE_DEBUGGABLE void FPBDCollisionSolver::SolvePositionNoFriction(
			const FSolverReal Dt,
			const FSolverReal MaxPushOut)
		{
			// Apply the position correction so that all contacts have zero separation
			for (int32 PointIndex = 0; PointIndex < NumManifoldPoints(); ++PointIndex)
			{
				FPBDCollisionSolverManifoldPoint& SolverManifoldPoint = State.ManifoldPoints[PointIndex];
				if (!!SolverManifoldPoint.Flags.bDisablePositionSolve)
				{
					continue;
				}

				FSolverReal ContactCorrectionNormal;
				CalculateContactPositionCorrectionNormal(SolverManifoldPoint, MaxPushOut, ContactCorrectionNormal);

				// Shift the contact to the hard shell under the soft layer
				const FSolverReal HardContactErrorNormal = SolverManifoldPoint.ContactDeltaNormal + ContactCorrectionNormal - State.SoftPhi;

				const bool bProcessManifoldPoint = (HardContactErrorNormal < FSolverReal(0)) || (SolverManifoldPoint.NetPushOutNormal > FSolverReal(UE_SMALL_NUMBER));
				if (bProcessManifoldPoint)
				{
					ApplyPositionCorrectionNormal(
						State.Stiffness,
						HardContactErrorNormal,
						SolverManifoldPoint);
				}

				//
				// TODO: when State.SoftPhi < 0, apply soft position correction
				//
			}
		}

		/**
		 * @brief Calculate and apply the velocity correction for this iteration
		 * @return true if we need to run more iterations, false if we did not apply any correction
		*/
		FORCEINLINE_DEBUGGABLE void FPBDCollisionSolver::SolveVelocity(
			const FSolverReal Dt,
			const bool bApplyDynamicFriction)
		{
			// Apply restitution at the average contact point
			// This means we don't need to run as many iterations to get stable bouncing
			// It also helps with zero restitution to counter any velocioty added by the PBD solve
			const bool bSolveAverageContact = (NumManifoldPoints() > 1) && CVars::bChaos_PBDCollisionSolver_Velocity_AveragePointEnabled;
			if (bSolveAverageContact)
			{
				SolveVelocityAverage(Dt);
			}

			const FSolverReal DynamicFriction = (bApplyDynamicFriction && (Dt > 0) && CVars::bChaos_PBDCollisionSolver_Velocity_FrictionEnabled) ? State.VelocityFriction : FSolverReal(0);

			for (int32 PointIndex = 0; PointIndex < NumManifoldPoints(); ++PointIndex)
			{
				FPBDCollisionSolverManifoldPoint& SolverManifoldPoint = State.ManifoldPoints[PointIndex];
				if (!!SolverManifoldPoint.Flags.bDisableVelocitySolve)
				{
					continue;
				}

				// Solver for normal velocity when...
				// - we applied a positional impulse
				// - we skipped the position solve, but not the velocity solve
				// - we are in split impulse mode and have an overlap
				const bool bShouldSolveNormalVelocity = 
					(SolverManifoldPoint.NetPushOutNormal > FSolverReal(0)) 
					|| !!SolverManifoldPoint.Flags.bDisablePositionSolve
					|| ((SolverManifoldPoint.SplitImpulseAlpha > 0) && (SolverManifoldPoint.ContactDeltaNormal < 0.0f));

				// Solver for tangential velocity if we have non-zero friction and a position correction
				// (unless the position solve step is disabled, in which case we assume we want to run the velocity solve)
				const bool bShouldSolveTangentVelocity = 
					(DynamicFriction > 0)
					&& (
						bShouldSolveNormalVelocity 
						|| (SolverManifoldPoint.ContactDeltaNormal < State.SoftPhi) 
						|| (SolverManifoldPoint.NetSoftPushOutNormal > FSolverReal(0)) 
					) 
					&& !SolverManifoldPoint.Flags.bDisableFriction;

				if (bShouldSolveNormalVelocity || bShouldSolveTangentVelocity)
				{
					const FSolverReal MinImpulseNormal = FMath::Min(FSolverReal(0), -(SolverManifoldPoint.NetPushOutNormal + SolverManifoldPoint.NetSoftPushOutNormal) / Dt);

					// @todo(chaos): clean this up - friction and no-friction versions should share solve-normal code
					if (bShouldSolveTangentVelocity)
					{
						FSolverReal ContactVelocityDeltaNormal, ContactVelocityDeltaTangentU, ContactVelocityDeltaTangentV;
						CalculateContactVelocityError(SolverManifoldPoint, DynamicFriction, Dt, ContactVelocityDeltaNormal, ContactVelocityDeltaTangentU, ContactVelocityDeltaTangentV);

						// 1D friction: zero out the V component. TangentU is already velocity-aligned
						if (CVars::bChaos_PBDCollisionSolver_Use1DFriction)
						{
							ContactVelocityDeltaTangentV = FSolverReal(0);
						}

						if (!bShouldSolveNormalVelocity)
						{
							ContactVelocityDeltaNormal = 0;
						}

						ApplyVelocityCorrection(
							State.Stiffness,
							Dt,
							DynamicFriction,
							ContactVelocityDeltaNormal,
							ContactVelocityDeltaTangentU,
							ContactVelocityDeltaTangentV,
							MinImpulseNormal,
							SolverManifoldPoint);
					}
					else if (bShouldSolveNormalVelocity)
					{
						FSolverReal ContactVelocityDeltaNormal;
						CalculateContactVelocityErrorNormal(SolverManifoldPoint, ContactVelocityDeltaNormal);

						ApplyVelocityCorrectionNormal(
							State.Stiffness,
							ContactVelocityDeltaNormal,
							MinImpulseNormal,
							SolverManifoldPoint);
					}
				}
			}
		}

	}	// namespace Private
}	// namespace Chaos

