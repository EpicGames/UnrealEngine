// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/Collision/ContactPoint.h"
#include "Chaos/Defines.h"
#include "Chaos/Evolution/SolverBody.h"
#include "Chaos/Framework/UncheckedArray.h"

namespace Chaos
{
	class FManifoldPoint;
	class FPBDCollisionConstraint;

	namespace Private
	{
		class FPBDCollisionSolverJacobi;
	}

	namespace CVars
	{
		extern bool bChaos_PBDCollisionSolver_Velocity_FrictionEnabled;
		extern float Chaos_PBDCollisionSolver_Position_StaticFrictionStiffness;
		extern float Chaos_PBDCollisionSolver_JacobiStiffness;
		extern float Chaos_PBDCollisionSolver_JacobiPositionTolerance;
		extern float Chaos_PBDCollisionSolver_JacobiRotationTolerance;
	}

	namespace Private
	{
		class FSolverVec3SOA
		{
			AlignedFloat4 VX;
			AlignedFloat4 VY;
			AlignedFloat4 VZ;
		};

		class FSolverRealSOA
		{
			AlignedFloat4 V;
		};

		/**
		 * @brief A set of 4 manifold points in a FPBDCollisionSolver
		*/
		class FPBDCollisionSolverJacobiManifoldPoints
		{
		public:
			static const int32 MaxManifoldPoints = 4;

			void Reset()
			{
				ManifoldPoints.Reset();
			}

			FORCEINLINE_DEBUGGABLE int32 NumManifoldPoints() const
			{
				return ManifoldPoints.Num();
			}

			FORCEINLINE_DEBUGGABLE int32 AddManifoldPoint()
			{
				return ManifoldPoints.AddUninitialized();
			}

			FORCEINLINE_DEBUGGABLE void SetWorldContact(
				const int32 ManifoldPointIndex,
				const FSolverVec3& InRelativeContactPosition0,
				const FSolverVec3& InRelativeContactPosition1,
				const FSolverVec3& InWorldContactNormal,
				const FSolverVec3& InWorldContactTangentU,
				const FSolverVec3& InWorldContactTangentV,
				const FSolverReal InWorldContactDeltaNormal,
				const FSolverReal InWorldContactDeltaTangentU,
				const FSolverReal InWorldContactDeltaTangentV,
				const FSolverReal InWorldContactVelocityTargetNormal)
			{
				ManifoldPoints[ManifoldPointIndex].RelativeContactPosition0 = InRelativeContactPosition0;
				ManifoldPoints[ManifoldPointIndex].RelativeContactPosition1 = InRelativeContactPosition1;
				ManifoldPoints[ManifoldPointIndex].ContactNormal = InWorldContactNormal;
				ManifoldPoints[ManifoldPointIndex].ContactTangentU = InWorldContactTangentU;
				ManifoldPoints[ManifoldPointIndex].ContactTangentV = InWorldContactTangentV;
				ManifoldPoints[ManifoldPointIndex].ContactDeltaNormal = InWorldContactDeltaNormal;
				ManifoldPoints[ManifoldPointIndex].ContactDeltaTangentU = InWorldContactDeltaTangentU;
				ManifoldPoints[ManifoldPointIndex].ContactDeltaTangentV = InWorldContactDeltaTangentV;
				ManifoldPoints[ManifoldPointIndex].ContactTargetVelocityNormal = InWorldContactVelocityTargetNormal;
			}

			/**
			 * @brief Initialize the geometric data for the contact
			*/
			FORCEINLINE_DEBUGGABLE void InitContact(
				const int32 ManifoldPointIndex,
				const FConstraintSolverBody& Body0,
				const FConstraintSolverBody& Body1)
			{
				ManifoldPoints[ManifoldPointIndex].NetPushOutNormal = FSolverReal(0);
				ManifoldPoints[ManifoldPointIndex].NetPushOutTangentU = FSolverReal(0);
				ManifoldPoints[ManifoldPointIndex].NetPushOutTangentV = FSolverReal(0);
				ManifoldPoints[ManifoldPointIndex].NetImpulseNormal = FSolverReal(0);
				ManifoldPoints[ManifoldPointIndex].NetImpulseTangentU = FSolverReal(0);
				ManifoldPoints[ManifoldPointIndex].NetImpulseTangentV = FSolverReal(0);
				ManifoldPoints[ManifoldPointIndex].StaticFrictionRatio = FSolverReal(0);

				UpdateMass(ManifoldPointIndex, Body0, Body1);
			}

			/**
			 * @brief Update the cached mass properties based on the current body transforms
			*/
			void UpdateMass(
				const int32 ManifoldPointIndex, 
				const FConstraintSolverBody& Body0, 
				const FConstraintSolverBody& Body1)
			{
				FSolverReal ContactMassInvNormal = FSolverReal(0);
				FSolverReal ContactMassInvTangentU = FSolverReal(0);
				FSolverReal ContactMassInvTangentV = FSolverReal(0);

				if (Body0.IsDynamic())
				{
					const FSolverVec3 R0xN = FSolverVec3::CrossProduct(ManifoldPoints[ManifoldPointIndex].RelativeContactPosition0, ManifoldPoints[ManifoldPointIndex].ContactNormal);
					const FSolverVec3 R0xU = FSolverVec3::CrossProduct(ManifoldPoints[ManifoldPointIndex].RelativeContactPosition0, ManifoldPoints[ManifoldPointIndex].ContactTangentU);
					const FSolverVec3 R0xV = FSolverVec3::CrossProduct(ManifoldPoints[ManifoldPointIndex].RelativeContactPosition0, ManifoldPoints[ManifoldPointIndex].ContactTangentV);

					const FSolverMatrix33 InvI0 = Body0.InvI();

					ManifoldPoints[ManifoldPointIndex].ContactNormalAngular0 = InvI0 * R0xN;
					ManifoldPoints[ManifoldPointIndex].ContactTangentUAngular0 = InvI0 * R0xU;
					ManifoldPoints[ManifoldPointIndex].ContactTangentVAngular0 = InvI0 * R0xV;

					ContactMassInvNormal += FSolverVec3::DotProduct(R0xN, ManifoldPoints[ManifoldPointIndex].ContactNormalAngular0) + Body0.InvM();
					ContactMassInvTangentU += FSolverVec3::DotProduct(R0xU, ManifoldPoints[ManifoldPointIndex].ContactTangentUAngular0) + Body0.InvM();
					ContactMassInvTangentV += FSolverVec3::DotProduct(R0xV, ManifoldPoints[ManifoldPointIndex].ContactTangentVAngular0) + Body0.InvM();
				}
				if (Body1.IsDynamic())
				{
					const FSolverVec3 R1xN = FSolverVec3::CrossProduct(ManifoldPoints[ManifoldPointIndex].RelativeContactPosition1, ManifoldPoints[ManifoldPointIndex].ContactNormal);
					const FSolverVec3 R1xU = FSolverVec3::CrossProduct(ManifoldPoints[ManifoldPointIndex].RelativeContactPosition1, ManifoldPoints[ManifoldPointIndex].ContactTangentU);
					const FSolverVec3 R1xV = FSolverVec3::CrossProduct(ManifoldPoints[ManifoldPointIndex].RelativeContactPosition1, ManifoldPoints[ManifoldPointIndex].ContactTangentV);

					const FSolverMatrix33 InvI1 = Body1.InvI();

					ManifoldPoints[ManifoldPointIndex].ContactNormalAngular1 = InvI1 * R1xN;
					ManifoldPoints[ManifoldPointIndex].ContactTangentUAngular1 = InvI1 * R1xU;
					ManifoldPoints[ManifoldPointIndex].ContactTangentVAngular1 = InvI1 * R1xV;

					ContactMassInvNormal += FSolverVec3::DotProduct(R1xN, ManifoldPoints[ManifoldPointIndex].ContactNormalAngular1) + Body1.InvM();
					ContactMassInvTangentU += FSolverVec3::DotProduct(R1xU, ManifoldPoints[ManifoldPointIndex].ContactTangentUAngular1) + Body1.InvM();
					ContactMassInvTangentV += FSolverVec3::DotProduct(R1xV, ManifoldPoints[ManifoldPointIndex].ContactTangentVAngular1) + Body1.InvM();
				}

				ManifoldPoints[ManifoldPointIndex].ContactMassNormal = (ContactMassInvNormal > FSolverReal(UE_SMALL_NUMBER)) ? FSolverReal(1) / ContactMassInvNormal : FSolverReal(0);
				ManifoldPoints[ManifoldPointIndex].ContactMassTangentU = (ContactMassInvTangentU > FSolverReal(UE_SMALL_NUMBER)) ? FSolverReal(1) / ContactMassInvTangentU : FSolverReal(0);
				ManifoldPoints[ManifoldPointIndex].ContactMassTangentV = (ContactMassInvTangentV > FSolverReal(UE_SMALL_NUMBER)) ? FSolverReal(1) / ContactMassInvTangentV : FSolverReal(0);
			}

			/**
			 * @brief Update the contact mass for the normal correction
			 * This is used by shock propagation.
			*/
			void UpdateMassNormal(
				const int32 ManifoldPointIndex, 
				const FConstraintSolverBody& Body0, 
				const FConstraintSolverBody& Body1)
			{
				FSolverReal ContactMassInvNormal = FSolverReal(0);
				if (Body0.IsDynamic())
				{
					const FSolverVec3 R0xN = FSolverVec3::CrossProduct(ManifoldPoints[ManifoldPointIndex].RelativeContactPosition0, ManifoldPoints[ManifoldPointIndex].ContactNormal);
					const FSolverMatrix33 InvI0 = Body0.InvI();
					ManifoldPoints[ManifoldPointIndex].ContactNormalAngular0 = InvI0 * R0xN;
					ContactMassInvNormal += FSolverVec3::DotProduct(R0xN, ManifoldPoints[ManifoldPointIndex].ContactNormalAngular0) + Body0.InvM();
				}
				if (Body1.IsDynamic())
				{
					const FSolverVec3 R1xN = FSolverVec3::CrossProduct(ManifoldPoints[ManifoldPointIndex].RelativeContactPosition1, ManifoldPoints[ManifoldPointIndex].ContactNormal);
					const FSolverMatrix33 InvI1 = Body1.InvI();
					ManifoldPoints[ManifoldPointIndex].ContactNormalAngular1 = InvI1 * R1xN;
					ContactMassInvNormal += FSolverVec3::DotProduct(R1xN, ManifoldPoints[ManifoldPointIndex].ContactNormalAngular1) + Body1.InvM();
				}
				ManifoldPoints[ManifoldPointIndex].ContactMassNormal = (ContactMassInvNormal > FSolverReal(UE_SMALL_NUMBER)) ? FSolverReal(1) / ContactMassInvNormal : FSolverReal(0);
			}

			FORCEINLINE_DEBUGGABLE FSolverVec3 GetNetPushOut(const int32 ManifoldPointIndex) const
			{
				return ManifoldPoints[ManifoldPointIndex].NetPushOutNormal* ManifoldPoints[ManifoldPointIndex].ContactNormal +
					ManifoldPoints[ManifoldPointIndex].NetPushOutTangentU * ManifoldPoints[ManifoldPointIndex].ContactTangentU +
					ManifoldPoints[ManifoldPointIndex].NetPushOutTangentV * ManifoldPoints[ManifoldPointIndex].ContactTangentV;
			}

			FORCEINLINE_DEBUGGABLE FSolverVec3 GetNetImpulse(const int32 ManifoldPointIndex) const
			{
				return ManifoldPoints[ManifoldPointIndex].NetImpulseNormal* ManifoldPoints[ManifoldPointIndex].ContactNormal +
					ManifoldPoints[ManifoldPointIndex].NetImpulseTangentU * ManifoldPoints[ManifoldPointIndex].ContactTangentU +
					ManifoldPoints[ManifoldPointIndex].NetImpulseTangentV * ManifoldPoints[ManifoldPointIndex].ContactTangentV;
			}

			FORCEINLINE_DEBUGGABLE FSolverReal GetStaticFrictionRatio(const int32 ManifoldPointIndex) const
			{
				return ManifoldPoints[ManifoldPointIndex].StaticFrictionRatio;
			}

			/**
			 * @brief Calculate the position error at the current transforms
			 * @param MaxPushOut a limit on the position error for this iteration to prevent initial-penetration explosion (a common PBD problem)
			*/
			FORCEINLINE_DEBUGGABLE void CalculateContactPositionErrorNormal(
				const int32 ManifoldPointIndex, 
				const FConstraintSolverBody& Body0, 
				const FConstraintSolverBody& Body1, 
				const FSolverReal MaxPushOut, 
				FSolverReal& OutContactDeltaNormal) const
			{
				// Linear version: calculate the contact delta assuming linear motion after applying a positional impulse at the contact point. There will be an error that depends on the size of the rotation.
				const FSolverVec3 ContactDelta0 = Body0.DP() + FSolverVec3::CrossProduct(Body0.DQ(), ManifoldPoints[ManifoldPointIndex].RelativeContactPosition0);
				const FSolverVec3 ContactDelta1 = Body1.DP() + FSolverVec3::CrossProduct(Body1.DQ(), ManifoldPoints[ManifoldPointIndex].RelativeContactPosition1);
				const FSolverVec3 ContactDelta = ContactDelta0 - ContactDelta1;
				OutContactDeltaNormal = ManifoldPoints[ManifoldPointIndex].ContactDeltaNormal + FSolverVec3::DotProduct(ContactDelta, ManifoldPoints[ManifoldPointIndex].ContactNormal);

				// NOTE: OutContactDeltaNormal is negative for penetration
				// NOTE: MaxPushOut == 0 disables the pushout limits
				if ((MaxPushOut > 0) && (OutContactDeltaNormal < -MaxPushOut))
				{
					OutContactDeltaNormal = -MaxPushOut;
				}
			}

			FORCEINLINE_DEBUGGABLE void CalculateContactPositionErrorTangential(
				const int32 ManifoldPointIndex, 
				const FConstraintSolverBody& Body0, 
				const FConstraintSolverBody& Body1, 
				FSolverReal& OutContactDeltaTangentU, 
				FSolverReal& OutContactDeltaTangentV) const
			{
				// Linear version: calculate the contact delta assuming linear motion after applying a positional impulse at the contact point. There will be an error that depends on the size of the rotation.
				const FSolverVec3 ContactDelta0 = Body0.DP() + FSolverVec3::CrossProduct(Body0.DQ(), ManifoldPoints[ManifoldPointIndex].RelativeContactPosition0);
				const FSolverVec3 ContactDelta1 = Body1.DP() + FSolverVec3::CrossProduct(Body1.DQ(), ManifoldPoints[ManifoldPointIndex].RelativeContactPosition1);
				const FSolverVec3 ContactDelta = ContactDelta0 - ContactDelta1;
				OutContactDeltaTangentU = ManifoldPoints[ManifoldPointIndex].ContactDeltaTangentU + FSolverVec3::DotProduct(ContactDelta, ManifoldPoints[ManifoldPointIndex].ContactTangentU);
				OutContactDeltaTangentV = ManifoldPoints[ManifoldPointIndex].ContactDeltaTangentV + FSolverVec3::DotProduct(ContactDelta, ManifoldPoints[ManifoldPointIndex].ContactTangentV);
			}

			FORCEINLINE_DEBUGGABLE void CalculatePositionCorrectionNormal(
				const int32 ManifoldPointIndex,
				const FSolverReal Stiffness,
				FSolverReal ContactDeltaNormal,
				FConstraintSolverBody& Body0,
				FConstraintSolverBody& Body1,
				FSolverVec3& InOutDX0,
				FSolverVec3& InOutDR0,
				FSolverVec3& InOutDX1,
				FSolverVec3& InOutDR1)
			{
				FSolverReal PushOutNormal = -Stiffness * ContactDeltaNormal * ManifoldPoints[ManifoldPointIndex].ContactMassNormal;

				// The total pushout so far this sub-step
				// Unilateral constraint: Net-negative impulses not allowed (negative incremental impulses are allowed as long as the net is positive)
				if ((ManifoldPoints[ManifoldPointIndex].NetPushOutNormal + PushOutNormal) > FSolverReal(0))
				{
					ManifoldPoints[ManifoldPointIndex].NetPushOutNormal += PushOutNormal;
				}
				else
				{
					PushOutNormal = -ManifoldPoints[ManifoldPointIndex].NetPushOutNormal;
					ManifoldPoints[ManifoldPointIndex].NetPushOutNormal = 0;
				}

				// Update the particle state based on the pushout
				if (Body0.IsDynamic())
				{
					InOutDX0 += (Body0.InvM() * PushOutNormal) * ManifoldPoints[ManifoldPointIndex].ContactNormal;
					InOutDR0 += ManifoldPoints[ManifoldPointIndex].ContactNormalAngular0 * PushOutNormal;
				}
				if (Body1.IsDynamic())
				{
					InOutDX1 -= (Body1.InvM() * PushOutNormal) * ManifoldPoints[ManifoldPointIndex].ContactNormal;
					InOutDR1 -= ManifoldPoints[ManifoldPointIndex].ContactNormalAngular1 * PushOutNormal;
				}
			}

			FORCEINLINE_DEBUGGABLE void ApplyFrictionCone(
				const int32 ManifoldPointIndex,
				const FSolverReal StaticFriction,
				const FSolverReal DynamicFriction,
				const FSolverReal MaxFrictionPushOut,
				FSolverReal& InOutPushOutTangentU,
				FSolverReal& InOutPushOutTangentV)
			{
				// Assume we stay in the friction cone...
				ManifoldPoints[ManifoldPointIndex].StaticFrictionRatio = FSolverReal(1);

				if (MaxFrictionPushOut < FSolverReal(UE_KINDA_SMALL_NUMBER))
				{
					// Note: we have already added the current iteration's PushOut to the NetPushOut but it has not been applied to the body
					// so we must subtract it again to calculate the actual pushout we want to undo (i.e., the net pushout that has been applied 
					// to the body so far from previous iterations)
					InOutPushOutTangentU = -(ManifoldPoints[ManifoldPointIndex].NetPushOutTangentU - InOutPushOutTangentU);
					InOutPushOutTangentV = -(ManifoldPoints[ManifoldPointIndex].NetPushOutTangentV - InOutPushOutTangentV);
					ManifoldPoints[ManifoldPointIndex].NetPushOutTangentU = FSolverReal(0);
					ManifoldPoints[ManifoldPointIndex].NetPushOutTangentV = FSolverReal(0);
					ManifoldPoints[ManifoldPointIndex].StaticFrictionRatio = FSolverReal(0);
				}
				else
				{
					// If we exceed the static friction cone, clip to the dynamic friction cone
					const FSolverReal MaxStaticPushOutTangentSq = FMath::Square(StaticFriction * MaxFrictionPushOut);
					const FSolverReal NetPushOutTangentSq = FMath::Square(ManifoldPoints[ManifoldPointIndex].NetPushOutTangentU) + FMath::Square(ManifoldPoints[ManifoldPointIndex].NetPushOutTangentV);
					if (NetPushOutTangentSq > MaxStaticPushOutTangentSq)
					{
						const FSolverReal MaxDynamicPushOutTangent = DynamicFriction * MaxFrictionPushOut;
						const FSolverReal FrictionMultiplier = MaxDynamicPushOutTangent * FMath::InvSqrt(NetPushOutTangentSq);
						const FSolverReal NetPushOutTangentU = FrictionMultiplier * ManifoldPoints[ManifoldPointIndex].NetPushOutTangentU;
						const FSolverReal NetPushOutTangentV = FrictionMultiplier * ManifoldPoints[ManifoldPointIndex].NetPushOutTangentV;
						InOutPushOutTangentU = NetPushOutTangentU - (ManifoldPoints[ManifoldPointIndex].NetPushOutTangentU - InOutPushOutTangentU);
						InOutPushOutTangentV = NetPushOutTangentV - (ManifoldPoints[ManifoldPointIndex].NetPushOutTangentV - InOutPushOutTangentV);
						ManifoldPoints[ManifoldPointIndex].NetPushOutTangentU = NetPushOutTangentU;
						ManifoldPoints[ManifoldPointIndex].NetPushOutTangentV = NetPushOutTangentV;
						ManifoldPoints[ManifoldPointIndex].StaticFrictionRatio = FrictionMultiplier;
					}
				}
			}

			FORCEINLINE_DEBUGGABLE void SolvePositionNoFriction(
				const FSolverReal Stiffness,
				const FSolverReal MaxPushOut,
				FConstraintSolverBody& Body0,
				FConstraintSolverBody& Body1)
			{
				// Apply the position correction so that all contacts have zero separation
				FSolverVec3 DX0 = FSolverVec3(0);
				FSolverVec3 DX1 = FSolverVec3(0);
				FSolverVec3 DR0 = FSolverVec3(0);
				FSolverVec3 DR1 = FSolverVec3(0);

				for (int32 ManifoldPointIndex = 0; ManifoldPointIndex < NumManifoldPoints(); ++ManifoldPointIndex)
				{
					FSolverReal ContactDeltaNormal;
					CalculateContactPositionErrorNormal(ManifoldPointIndex, Body0.SolverBody(), Body1.SolverBody(), MaxPushOut, ContactDeltaNormal);

					const bool bProcessManifoldPoint = (ContactDeltaNormal < FSolverReal(0)) || (ManifoldPoints[ManifoldPointIndex].NetPushOutNormal > FSolverReal(UE_SMALL_NUMBER));
					if (bProcessManifoldPoint)
					{
						CalculatePositionCorrectionNormal(
							ManifoldPointIndex,
							Stiffness,
							ContactDeltaNormal,
							Body0,
							Body1,
							DX0, DR0, DX1, DR1);
					}
				}

				if (Body0.IsDynamic())
				{
					Body0.ApplyPositionDelta(DX0);
					Body0.ApplyRotationDelta(DR0);
				}
				if (Body1.IsDynamic())
				{
					Body1.ApplyPositionDelta(DX1);
					Body1.ApplyRotationDelta(DR1);
				}
			}

			FORCEINLINE_DEBUGGABLE bool SolvePositionWithFriction(
				const FSolverReal Stiffness,
				const FSolverReal Dt,
				const FSolverReal StaticFriction,
				const FSolverReal DynamicFriction,
				const FSolverReal MaxPushOut,
				FConstraintSolverBody& Body0,
				FConstraintSolverBody& Body1)
			{
				// Accumulate net pushout for friction limits below
				bool bApplyFriction[MaxManifoldPoints] = { false, };
				int32 NumFrictionContacts = 0;
				FSolverReal TotalPushOutNormal = FSolverReal(0);

				FSolverVec3 DX0 = FSolverVec3(0);
				FSolverVec3 DX1 = FSolverVec3(0);
				FSolverVec3 DR0 = FSolverVec3(0);
				FSolverVec3 DR1 = FSolverVec3(0);

				// Apply the position correction along the normal and determine if we want to run friction on each point
				for (int32 ManifoldPointIndex = 0; ManifoldPointIndex < NumManifoldPoints(); ++ManifoldPointIndex)
				{
					FSolverReal ContactDeltaNormal;
					CalculateContactPositionErrorNormal(ManifoldPointIndex, Body0.SolverBody(), Body1.SolverBody(), MaxPushOut, ContactDeltaNormal);

					// Apply a normal correction if we still have penetration or if we are now separated but have previously applied a correction that we may want to undo
					const bool bProcessManifoldPoint = (ContactDeltaNormal < FSolverReal(0)) || (ManifoldPoints[ManifoldPointIndex].NetPushOutNormal > FSolverReal(UE_SMALL_NUMBER));
					if (bProcessManifoldPoint)
					{
						CalculatePositionCorrectionNormal(
							ManifoldPointIndex,
							Stiffness,
							ContactDeltaNormal,
							Body0,
							Body1,
							DX0, DR0, DX1, DR1);

						TotalPushOutNormal += ManifoldPoints[ManifoldPointIndex].NetPushOutNormal;
					}

					// Friction gets updated for any point with a net normal correction or where we have previously had a normal correction and 
					// already applied friction (in which case we may need to zero it)
					if ((ManifoldPoints[ManifoldPointIndex].NetPushOutNormal != 0) || (ManifoldPoints[ManifoldPointIndex].NetPushOutTangentU != 0) || (ManifoldPoints[ManifoldPointIndex].NetPushOutTangentV != 0))
					{
						bApplyFriction[ManifoldPointIndex] = true;
						++NumFrictionContacts;
					}
				}

				// Apply the tangential position correction if required
				if (NumFrictionContacts > 0)
				{
					// We clip the tangential correction at each contact to the friction cone, but we use to average impulse
					// among all contacts as the clipping limit. This is not really correct but it is much more stable to 
					// differences in contacts from tick to tick
					const FSolverReal FrictionMaxPushOut = TotalPushOutNormal / FSolverReal(NumFrictionContacts);
					const FSolverReal FrictionStiffness = Stiffness * CVars::Chaos_PBDCollisionSolver_Position_StaticFrictionStiffness;

					for (int32 ManifoldPointIndex = 0; ManifoldPointIndex < NumManifoldPoints(); ++ManifoldPointIndex)
					{
						if (bApplyFriction[ManifoldPointIndex])
						{
							FSolverReal ContactDeltaTangentU, ContactDeltaTangentV;
							CalculateContactPositionErrorTangential(ManifoldPointIndex, Body0.SolverBody(), Body1.SolverBody(), ContactDeltaTangentU, ContactDeltaTangentV);

							// Bilateral constraint - negative values allowed (unlike the normal correction)
							FSolverReal PushOutTangentU = -Stiffness * ManifoldPoints[ManifoldPointIndex].ContactMassTangentU * ContactDeltaTangentU;
							FSolverReal PushOutTangentV = -Stiffness * ManifoldPoints[ManifoldPointIndex].ContactMassTangentV * ContactDeltaTangentV;

							ManifoldPoints[ManifoldPointIndex].NetPushOutTangentU += PushOutTangentU;
							ManifoldPoints[ManifoldPointIndex].NetPushOutTangentV += PushOutTangentV;

							ApplyFrictionCone(
								ManifoldPointIndex,
								StaticFriction,
								DynamicFriction,
								FrictionMaxPushOut,
								PushOutTangentU,			// InOut
								PushOutTangentV);			// InOut

							// Update the particle state based on the pushout
							const FSolverVec3 PushOut = PushOutTangentU * ManifoldPoints[ManifoldPointIndex].ContactTangentU + PushOutTangentV * ManifoldPoints[ManifoldPointIndex].ContactTangentV;
							if (Body0.IsDynamic())
							{
								DX0 += Body0.InvM() * PushOut;
								DR0 += ManifoldPoints[ManifoldPointIndex].ContactTangentUAngular0 * PushOutTangentU + ManifoldPoints[ManifoldPointIndex].ContactTangentVAngular0 * PushOutTangentV;
							}
							if (Body1.IsDynamic())
							{
								DX1 -= Body1.InvM() * PushOut;
								DR1 -= ManifoldPoints[ManifoldPointIndex].ContactTangentUAngular1 * PushOutTangentU + ManifoldPoints[ManifoldPointIndex].ContactTangentVAngular1 * PushOutTangentV;
							}
						}
					}

					if (Body0.IsDynamic())
					{
						Body0.ApplyPositionDelta(DX0);
						Body0.ApplyRotationDelta(DR0);
					}
					if (Body1.IsDynamic())
					{
						Body1.ApplyPositionDelta(DX1);
						Body1.ApplyRotationDelta(DR1);
					}
				}

				return false;
			}

			/**
			 * @brief Calculate the velocity error at the current transforms
			*/
			FORCEINLINE_DEBUGGABLE void CalculateContactVelocityErrorNormal(
				const int32 ManifoldPointIndex, 
				const FConstraintSolverBody& Body0, 
				const FConstraintSolverBody& Body1, 
				FSolverReal& OutContactVelocityDeltaNormal) const
			{
				const FSolverVec3 ContactVelocity0 = Body0.V() + FSolverVec3::CrossProduct(Body0.W(), ManifoldPoints[ManifoldPointIndex].RelativeContactPosition0);
				const FSolverVec3 ContactVelocity1 = Body1.V() + FSolverVec3::CrossProduct(Body1.W(), ManifoldPoints[ManifoldPointIndex].RelativeContactPosition1);
				const FSolverVec3 ContactVelocity = ContactVelocity0 - ContactVelocity1;
				const FSolverReal ContactVelocityNormal = FSolverVec3::DotProduct(ContactVelocity, ManifoldPoints[ManifoldPointIndex].ContactNormal);

				// Add up the errors in the velocity (current velocity - desired velocity)
				OutContactVelocityDeltaNormal = (ContactVelocityNormal - ManifoldPoints[ManifoldPointIndex].ContactTargetVelocityNormal);
			}

			FORCEINLINE_DEBUGGABLE void CalculateContactVelocityError(
				const int32 ManifoldPointIndex, 
				const FConstraintSolverBody& Body0, 
				const FConstraintSolverBody& Body1, 
				const FSolverReal DynamicFriction, 
				const FSolverReal Dt, 
				FSolverReal& OutContactVelocityDeltaNormal, 
				FSolverReal& OutContactVelocityDeltaTangent0, 
				FSolverReal& OutContactVelocityDeltaTangent1) const
			{
				const FSolverVec3 ContactVelocity0 = Body0.V() + FSolverVec3::CrossProduct(Body0.W(), ManifoldPoints[ManifoldPointIndex].RelativeContactPosition0);
				const FSolverVec3 ContactVelocity1 = Body1.V() + FSolverVec3::CrossProduct(Body1.W(), ManifoldPoints[ManifoldPointIndex].RelativeContactPosition1);
				const FSolverVec3 ContactVelocity = ContactVelocity0 - ContactVelocity1;
				const FSolverReal ContactVelocityNormal = FSolverVec3::DotProduct(ContactVelocity, ManifoldPoints[ManifoldPointIndex].ContactNormal);
				const FSolverReal ContactVelocityTangent0 = FSolverVec3::DotProduct(ContactVelocity, ManifoldPoints[ManifoldPointIndex].ContactTangentU);
				const FSolverReal ContactVelocityTangent1 = FSolverVec3::DotProduct(ContactVelocity, ManifoldPoints[ManifoldPointIndex].ContactTangentV);

				OutContactVelocityDeltaNormal = (ContactVelocityNormal - ManifoldPoints[ManifoldPointIndex].ContactTargetVelocityNormal);
				OutContactVelocityDeltaTangent0 = ContactVelocityTangent0;
				OutContactVelocityDeltaTangent1 = ContactVelocityTangent1;
			}

			// @todo(chaos): dynamic friction
			FORCEINLINE_DEBUGGABLE void SolveVelocity(
				const FSolverReal Stiffness,
				const FSolverReal Dt,
				const bool bApplyDynamicFriction,
				FConstraintSolverBody& Body0,
				FConstraintSolverBody& Body1)
			{
				FSolverVec3 DV0 = FSolverVec3(0);
				FSolverVec3 DV1 = FSolverVec3(0);
				FSolverVec3 DW0 = FSolverVec3(0);
				FSolverVec3 DW1 = FSolverVec3(0);

				for (int32 ManifoldPointIndex = 0; ManifoldPointIndex < NumManifoldPoints(); ++ManifoldPointIndex)
				{
					if (ShouldSolveVelocity(ManifoldPointIndex))
					{
						const FSolverReal MinImpulseNormal = FMath::Min(FSolverReal(0), -ManifoldPoints[ManifoldPointIndex].NetPushOutNormal / Dt);

						FSolverReal ContactVelocityDeltaNormal;
						CalculateContactVelocityErrorNormal(ManifoldPointIndex, Body0, Body1, ContactVelocityDeltaNormal);

						FSolverReal ImpulseNormal = -(Stiffness * ManifoldPoints[ManifoldPointIndex].ContactMassNormal) * ContactVelocityDeltaNormal;

						const FSolverReal NetImpulseNormal = ManifoldPoints[ManifoldPointIndex].NetImpulseNormal + ImpulseNormal;
						if (NetImpulseNormal < MinImpulseNormal)
						{
							ImpulseNormal = ImpulseNormal - NetImpulseNormal + MinImpulseNormal;
						}

						ManifoldPoints[ManifoldPointIndex].NetImpulseNormal += ImpulseNormal;

						FSolverVec3 Impulse = ImpulseNormal * ManifoldPoints[ManifoldPointIndex].ContactNormal;
						if (Body0.IsDynamic())
						{
							DV0 += Body0.InvM() * Impulse;
							DW0 += ManifoldPoints[ManifoldPointIndex].ContactNormalAngular0 * ImpulseNormal;
						}
						if (Body1.IsDynamic())
						{
							DV1 -= Body1.InvM() * Impulse;
							DW1 -= ManifoldPoints[ManifoldPointIndex].ContactNormalAngular1 * ImpulseNormal;
						}
					}
				}

				if (Body0.IsDynamic())
				{
					Body0.ApplyVelocityDelta(DV0, DW0);
				}
				if (Body1.IsDynamic())
				{
					Body1.ApplyVelocityDelta(DV1, DW1);
				}
			}

			// @todo(chaos): make private
		public:
			friend class FPBDCollisionSolverJacobi;

			/**
			 * @brief Whether we need to solve velocity for this manifold point (only if we were penetrating or applied a pushout)
			*/
			FORCEINLINE_DEBUGGABLE bool ShouldSolveVelocity(const int32 ManifoldPointIndex) const
			{
				// We ensure positive separating velocity for close contacts even if they didn't receive a pushout
				return (ManifoldPoints[ManifoldPointIndex].NetPushOutNormal > FSolverReal(0)) || (ManifoldPoints[ManifoldPointIndex].ContactDeltaNormal < FSolverReal(0));
			}

			struct FManifoldPoint
			{
				// World-space contact point relative to each particle's center of mass
				FSolverVec3 RelativeContactPosition0;
				FSolverVec3 RelativeContactPosition1;

				// World-space contact normal and tangents
				FSolverVec3 ContactNormal;
				FSolverVec3 ContactTangentU;
				FSolverVec3 ContactTangentV;

				// Errors to correct along each of the contact axes
				FSolverReal ContactDeltaNormal;
				FSolverReal ContactDeltaTangentU;
				FSolverReal ContactDeltaTangentV;

				// Target velocity along the normal direction
				FSolverReal ContactTargetVelocityNormal;

				// I^-1.(R x A) for each body where A is each axis (Normal, TangentU, TangentV)
				FSolverVec3 ContactNormalAngular0;
				FSolverVec3 ContactTangentUAngular0;
				FSolverVec3 ContactTangentVAngular0;
				FSolverVec3 ContactNormalAngular1;
				FSolverVec3 ContactTangentUAngular1;
				FSolverVec3 ContactTangentVAngular1;

				// Contact mass (for non-friction)
				FSolverReal ContactMassNormal;
				FSolverReal ContactMassTangentU;
				FSolverReal ContactMassTangentV;

				// Solver outputs
				FSolverReal NetPushOutNormal;
				FSolverReal NetPushOutTangentU;
				FSolverReal NetPushOutTangentV;
				FSolverReal NetImpulseNormal;
				FSolverReal NetImpulseTangentU;
				FSolverReal NetImpulseTangentV;

				// A measure of how much we exceeded the static friction threshold.
				// Equal to (NormalPushOut / TangentialPushOut) before clamping to the friction cone.
				// Used to move the static friction anchors to the edge of the cone in Scatter.
				FSolverReal StaticFrictionRatio;
			};

			TCArray<FManifoldPoint, MaxManifoldPoints> ManifoldPoints;
		};

		/**
		 * A Jocobi solver for the contact manifold between two shapes. This solves each
		 * manifold point independently and sums and dampens the net impulses before
		 * applying to the bodies.
		 * 
		 * Pros (vs Gauss Seidel):
		 *	-	If the contacts are symmetric, the impusles will be too. This fixed spurious rotations when 
		 *		a box with high restitution bounces exactly flat on the ground.
		 *	-	We can run the points in the manifold in parallel
		 *	-	Stacking is much more stable (but see below)
		 * Cons
		 *	-	We overestimate the impulse and must apply a damping factor to prevent energy gain. It is not
		 *		clear how we select the value of this damping - it is empirically tuned.
		 *	-	The damping factor leads to spongey stacks/piles
		 * 
		*/
		class FPBDCollisionSolverJacobi
		{
		public:
			static const int32 MaxConstrainedBodies = 2;
			static const int32 MaxPointsPerConstraint = 4;

			// Create a solver that is initialized to safe defaults
			static FPBDCollisionSolverJacobi MakeInitialized()
			{
				FPBDCollisionSolverJacobi Solver;
				Solver.State.Init();
				return Solver;
			}

			// Create a solver with no initialization
			static FPBDCollisionSolverJacobi MakeUninitialized()
			{
				return FPBDCollisionSolverJacobi();
			}

			// NOTE: Does not initialize any properties. See MakeInitialized
			FPBDCollisionSolverJacobi() {}

			/** Reset the state of the collision solver */
			FORCEINLINE_DEBUGGABLE void Reset()
			{
				State.SolverBodies[0].Reset();
				State.SolverBodies[1].Reset();
				ResetManifold();
			}

			FORCEINLINE_DEBUGGABLE void ResetManifold()
			{
				State.ManifoldPoints.Reset();
			}

			FORCEINLINE_DEBUGGABLE int32 AddManifoldPoint()
			{
				return State.ManifoldPoints.AddManifoldPoint();
			}

			FORCEINLINE_DEBUGGABLE FSolverReal StaticFriction() const
			{ 
				return State.StaticFriction;
			}

			FORCEINLINE_DEBUGGABLE FSolverReal DynamicFriction() const
			{ 
				return State.DynamicFriction; 
			}

			FORCEINLINE_DEBUGGABLE FSolverReal VelocityFriction() const
			{ 
				return State.VelocityFriction;
			}

			FORCEINLINE_DEBUGGABLE void SetFriction(const FSolverReal InStaticFriction, const FSolverReal InDynamicFriction, const FSolverReal InVelocityFriction)
			{
				State.StaticFriction = InStaticFriction;
				State.DynamicFriction = InDynamicFriction;
				State.VelocityFriction = InVelocityFriction;
			}

			FORCEINLINE_DEBUGGABLE void SetStiffness(const FSolverReal InStiffness)
			{
				State.Stiffness = InStiffness;
			}

			FORCEINLINE_DEBUGGABLE void SetSolverBodies(FSolverBody& SolverBody0, FSolverBody& SolverBody1)
			{
				State.SolverBodies[0].SetSolverBody(SolverBody0);
				State.SolverBodies[1].SetSolverBody(SolverBody1);
			}

			FORCEINLINE_DEBUGGABLE int32 NumManifoldPoints() const
			{
				return State.ManifoldPoints.NumManifoldPoints();
			}

			FORCEINLINE_DEBUGGABLE FSolverVec3 GetNetPushOut(const int32 ManifoldPointIndex) const
			{
				return State.ManifoldPoints.GetNetPushOut(ManifoldPointIndex);
			}

			FORCEINLINE_DEBUGGABLE FSolverVec3 GetNetImpulse(const int32 ManifoldPointIndex) const
			{
				return State.ManifoldPoints.GetNetImpulse(ManifoldPointIndex);
			}

			FORCEINLINE_DEBUGGABLE FSolverReal GetStaticFrictionRatio(const int32 ManifoldPointIndex) const
			{
				return State.ManifoldPoints.GetStaticFrictionRatio(ManifoldPointIndex);
			}

			/**
			 * Set up a manifold point (also calls FinalizeManifoldPoint)
			*/
			FORCEINLINE_DEBUGGABLE void SetManifoldPoint(
				const int32 ManifoldPointIndex,
				const FSolverVec3& InRelativeContactPosition0,
				const FSolverVec3& InRelativeContactPosition1,
				const FSolverVec3& InWorldContactNormal,
				const FSolverVec3& InWorldContactTangentU,
				const FSolverVec3& InWorldContactTangentV,
				const FSolverReal InWorldContactDeltaNormal,
				const FSolverReal InWorldContactDeltaTangentU,
				const FSolverReal InWorldContactDeltaTangentV,
				const FSolverReal InWorldContactVelocityTargetNormal)
			{
				InitManifoldPoint(
					ManifoldPointIndex,
					InRelativeContactPosition0,
					InRelativeContactPosition1,
					InWorldContactNormal,
					InWorldContactTangentU,
					InWorldContactTangentV,
					InWorldContactDeltaNormal,
					InWorldContactDeltaTangentU,
					InWorldContactDeltaTangentV,
					InWorldContactVelocityTargetNormal);

				FinalizeManifoldPoint(ManifoldPointIndex);
			}

			FORCEINLINE_DEBUGGABLE void InitManifoldPoint(
				const int32 ManifoldPointIndex,
				const FSolverVec3& InRelativeContactPosition0,
				const FSolverVec3& InRelativeContactPosition1,
				const FSolverVec3& InWorldContactNormal,
				const FSolverVec3& InWorldContactTangentU,
				const FSolverVec3& InWorldContactTangentV,
				const FSolverReal InWorldContactDeltaNormal,
				const FSolverReal InWorldContactDeltaTangentU,
				const FSolverReal InWorldContactDeltaTangentV,
				const FSolverReal InWorldContactVelocityTargetNormal)
			{
				State.ManifoldPoints.SetWorldContact(
					ManifoldPointIndex,
					InRelativeContactPosition0,
					InRelativeContactPosition1,
					InWorldContactNormal,
					InWorldContactTangentU,
					InWorldContactTangentV,
					InWorldContactDeltaNormal,
					InWorldContactDeltaTangentU,
					InWorldContactDeltaTangentV,
					InWorldContactVelocityTargetNormal
				);
			}

			/** 
			 * Finish manifold point setup.
			 * NOTE: Can only be called after the InitManifoldPoint has been called
			*/
			FORCEINLINE_DEBUGGABLE void FinalizeManifoldPoint(const int32 ManifoldPointIndex)
			{
				State.ManifoldPoints.InitContact(
					ManifoldPointIndex,
					State.SolverBodies[0],
					State.SolverBodies[1]);
			}

			/**
			 * @brief Get the first (decorated) solver body
			 * The decorator add a possible mass scale
			*/
			FORCEINLINE_DEBUGGABLE FConstraintSolverBody& SolverBody0() { return State.SolverBodies[0]; }
			FORCEINLINE_DEBUGGABLE const FConstraintSolverBody& SolverBody0() const { return State.SolverBodies[0]; }

			/**
			 * @brief Get the second (decorated) solver body
			 * The decorator add a possible mass scale
			*/
			FORCEINLINE_DEBUGGABLE FConstraintSolverBody& SolverBody1() { return State.SolverBodies[1]; }
			FORCEINLINE_DEBUGGABLE const FConstraintSolverBody& SolverBody1() const { return State.SolverBodies[1]; }

			/**
			 * @brief Set up the mass scaling for shock propagation, using the position-phase mass scale
			*/
			void EnablePositionShockPropagation();

			/**
			 * @brief Set up the mass scaling for shock propagation, using the velocity-phase mass scale
			*/
			void EnableVelocityShockPropagation();

			/**
			 * @brief Disable mass scaling
			*/
			void DisableShockPropagation();

			/**
			 * @brief Calculate and apply the position correction for this iteration
			 * @return true if we need to run more iterations, false if we did not apply any correction
			*/
			FORCEINLINE_DEBUGGABLE bool SolvePositionNoFriction(const FSolverReal Dt, const FSolverReal MaxPushOut)
			{
				// SolverBody decorator used to add mass scaling
				FConstraintSolverBody& Body0 = SolverBody0();
				FConstraintSolverBody& Body1 = SolverBody1();

				const FSolverReal Stiffness = CVars::Chaos_PBDCollisionSolver_JacobiStiffness * State.Stiffness;

				State.ManifoldPoints.SolvePositionNoFriction(
					Stiffness,
					MaxPushOut,
					Body0, 
					Body1);

				return false;
			}

			FORCEINLINE_DEBUGGABLE bool SolvePositionWithFriction(const FSolverReal Dt, const FSolverReal MaxPushOut)
			{
				// SolverBody decorator used to add mass scaling
				FConstraintSolverBody& Body0 = SolverBody0();
				FConstraintSolverBody& Body1 = SolverBody1();

				const FSolverReal Stiffness = CVars::Chaos_PBDCollisionSolver_JacobiStiffness * State.Stiffness;

				State.ManifoldPoints.SolvePositionWithFriction(
					Stiffness,
					Dt,
					State.StaticFriction,
					State.DynamicFriction,
					MaxPushOut,
					Body0,
					Body1);

				return false;
			}


			/**
			 * @brief Calculate and apply the velocity correction for this iteration
			 * @return true if we need to run more iterations, false if we did not apply any correction
			*/
			FORCEINLINE_DEBUGGABLE bool SolveVelocity(const FSolverReal Dt, const bool bApplyDynamicFriction)
			{
				FConstraintSolverBody& Body0 = SolverBody0();
				FConstraintSolverBody& Body1 = SolverBody1();

				const FSolverReal Stiffness = CVars::Chaos_PBDCollisionSolver_JacobiStiffness * State.Stiffness;

				State.ManifoldPoints.SolveVelocity(
					Stiffness,
					Dt, 
					bApplyDynamicFriction,
					Body0,
					Body1);

				return false;
			}

		private:
			/**
			 * @brief Apply the inverse mass scale the body with the lower level
			 * @param InvMassScale
			*/
			void SetShockPropagationInvMassScale(const FSolverReal InvMassScale);

			struct FState
			{
				FState()
				{
				}

				void Init()
				{
					SolverBodies[0].Init();
					SolverBodies[1].Init();
					StaticFriction = 0;
					DynamicFriction = 0;
					VelocityFriction = 0;
					Stiffness = 1;
					ManifoldPoints.Reset();
				}

				// Static Friction in the position-solve phase
				FSolverReal StaticFriction;

				// Dynamic Friction in the position-solve phase
				FSolverReal DynamicFriction;

				// Dynamic Friction in the velocity-solve phase
				FSolverReal VelocityFriction;

				// Solver stiffness (scales all pushout and impulses)
				FSolverReal Stiffness;

				// Bodies and contacts
				FConstraintSolverBody SolverBodies[MaxConstrainedBodies];
				FPBDCollisionSolverJacobiManifoldPoints ManifoldPoints;
			};

			FState State;
		};


		//////////////////////////////////////////////////////////////////////////////////////////////////
		//////////////////////////////////////////////////////////////////////////////////////////////////
		//////////////////////////////////////////////////////////////////////////////////////////////////
		//////////////////////////////////////////////////////////////////////////////////////////////////


		/**
		 * A helper for solving arrays of constraints
		 */
		class FPBDCollisionSolverJacobiHelper
		{
		public:
			static void SolvePositionNoFriction(const TArrayView<FPBDCollisionSolverJacobi>& CollisionSolvers, const FSolverReal Dt, const FSolverReal MaxPushOut);
			static void SolvePositionWithFriction(const TArrayView<FPBDCollisionSolverJacobi>& CollisionSolvers, const FSolverReal Dt, const FSolverReal MaxPushOut);
			static void SolveVelocity(const TArrayView<FPBDCollisionSolverJacobi>& CollisionSolvers, const FSolverReal Dt, const bool bApplyDynamicFriction);

			static void CheckISPC();
		};

	}	// namespace Private
}	// namespace Chaos

