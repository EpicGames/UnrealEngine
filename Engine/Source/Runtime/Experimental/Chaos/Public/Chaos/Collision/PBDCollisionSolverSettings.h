// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"

namespace Chaos
{
	/**
		* @brief Settings to control the low-level collision solver behaviour
	*/
	class FPBDCollisionSolverSettings
	{
	public:
		FPBDCollisionSolverSettings()
			: MaxPushOutVelocity(0)
			, DepenetrationVelocity(0)
			, NumPositionFrictionIterations(4)
			, NumVelocityFrictionIterations(1)
			, NumPositionShockPropagationIterations(3)
			, NumVelocityShockPropagationIterations(2)
			, bUseSplitImpulse(false)
		{
		}

		// Maximum speed at which two objects can depenetrate (actually, how much relative velocity can be added
		// to a contact per frame when depentrating. Stacks and deep penetrations can lead to larger velocities)
		// A value of zero means unlimited.
		FReal MaxPushOutVelocity;

		// The speed at which initially-overlapping objects depentrate.
		// This value is used when particles do not provide an override for their MaxDepenetrationVelocity.
		FRealSingle DepenetrationVelocity;

		// How many of the position iterations should run static/dynamic friction
		int32 NumPositionFrictionIterations;

		// How many of the velocity iterations should run dynamic friction 
		// This only applies to quadratic shapes if we have static friction enabled (NumPositionFrictionIterations > 0)
		// RBAN is the only system that uses velocity-based friction by default (it also disables static friction)
		// NOTE: velocity-based dynamic friction behaviour is iteration count dependent so generally want (NumVelocityFrictionIterations <= 1)
		// @todo(chaos): fix iteration count dependence of velocity-based friction, or remove it as a feature
		int32 NumVelocityFrictionIterations;

		// How many position iterations should have shock propagation enabled
		int32 NumPositionShockPropagationIterations;

		// How many velocity iterations should have shock propagation enabled
		int32 NumVelocityShockPropagationIterations;

		// Whether to use a split-impulse for the position correction. In split-impulse mode
		// the position correction is momentumless
		uint32 bUseSplitImpulse : 1;
	};

}	// namespace Chaos
