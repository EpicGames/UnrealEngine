// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/Collision/ContactPoint.h"
#include "Chaos/Defines.h"
#include "Chaos/Evolution/SolverBody.h"
#include "Chaos/Simd4.h"

// Set to 1 to use a dummy solver body instead of a nullptr in unused lanes, and an aligned Simd gather operation
// Currently this is slower, but once the object sizes are reduced may be better
#ifndef CHAOS_SIMDCOLLISIONSOLVER_USEDUMMYSOLVERBODY
#define CHAOS_SIMDCOLLISIONSOLVER_USEDUMMYSOLVERBODY 0
#endif

namespace Chaos
{
	class FManifoldPoint;
	class FPBDCollisionConstraint;

	namespace CVars
	{
		extern float Chaos_PBDCollisionSolver_Position_MinInvMassScale;
		extern float Chaos_PBDCollisionSolver_Velocity_MinInvMassScale;
	}

	namespace Private
	{
		template<int TNumLanes>
		class TPBDCollisionSolverSimd;

		template<int TNumLanes>
		using TSolverBodyPtrSimd = TSimdValue<FSolverBody*, TNumLanes>;

		inline void GatherBodyPositionCorrections(
			const TSolverBodyPtrSimd<4>& Body0,
			const TSolverBodyPtrSimd<4>& Body1,
			TSimdVec3f<4>& DP0,
			TSimdVec3f<4>& DQ0,
			TSimdVec3f<4>& DP1,
			TSimdVec3f<4>& DQ1)
		{
#if !CHAOS_SIMDCOLLISIONSOLVER_USEDUMMYSOLVERBODY
			// Non-SIMD gather
			for (int32 LaneIndex = 0; LaneIndex < 4; ++LaneIndex)
			{
				const FSolverBody* LaneBody0 = Body0.GetValue(LaneIndex);
				const FSolverBody* LaneBody1 = Body1.GetValue(LaneIndex);
				if (LaneBody0 != nullptr)
				{
					DP0.SetValue(LaneIndex, LaneBody0->DP());
					DQ0.SetValue(LaneIndex, LaneBody0->DQ());
				}
				if (LaneBody1 != nullptr)
				{
					DP1.SetValue(LaneIndex, LaneBody1->DP());
					DQ1.SetValue(LaneIndex, LaneBody1->DQ());
				}
			}
#else
			// SIMD gather (valid ptrs to dummy object for unused lanes)
			const FSolverBody* Body00 = Body0.GetValue(0);
			const FSolverBody* Body01 = Body0.GetValue(1);
			const FSolverBody* Body02 = Body0.GetValue(2);
			const FSolverBody* Body03 = Body0.GetValue(3);
			DP0 = SimdGatherAligned(
				Body00->DP(),
				Body01->DP(),
				Body02->DP(),
				Body03->DP());
			DQ0 = SimdGatherAligned(
				Body00->DQ(),
				Body01->DQ(),
				Body02->DQ(),
				Body03->DQ());

			const FSolverBody* Body10 = Body1.GetValue(0);
			const FSolverBody* Body11 = Body1.GetValue(1);
			const FSolverBody* Body12 = Body1.GetValue(2);
			const FSolverBody* Body13 = Body1.GetValue(3);
			DP1 = SimdGatherAligned(
				Body10->DP(),
				Body11->DP(),
				Body12->DP(),
				Body13->DP());
			DQ1 = SimdGatherAligned(
				Body10->DQ(),
				Body11->DQ(),
				Body12->DQ(),
				Body13->DQ());
#endif
		}

		inline void ScatterBodyPositionCorrections(
			const TSimdVec3f<4>& DP0,
			const TSimdVec3f<4>& DQ0,
			const TSimdVec3f<4>& DP1,
			const TSimdVec3f<4>& DQ1,
			const TSolverBodyPtrSimd<4>& Body0,
			const TSolverBodyPtrSimd<4>& Body1)
		{
			for (int32 LaneIndex = 0; LaneIndex < 4; ++LaneIndex)
			{
				FSolverBody* LaneBody0 = Body0.GetValue(LaneIndex);
				FSolverBody* LaneBody1 = Body1.GetValue(LaneIndex);
				if (LaneBody0 != nullptr)
				{
					LaneBody0->SetDP(DP0.GetValue(LaneIndex));
					LaneBody0->SetDQ(DQ0.GetValue(LaneIndex));
				}
				if (LaneBody1 != nullptr)
				{
					LaneBody1->SetDP(DP1.GetValue(LaneIndex));
					LaneBody1->SetDQ(DQ1.GetValue(LaneIndex));
				}
			}
		}


		inline void GatherBodyVelocities(
			const TSolverBodyPtrSimd<4>& Body0,
			const TSolverBodyPtrSimd<4>& Body1,
			TSimdVec3f<4>& V0,
			TSimdVec3f<4>& W0,
			TSimdVec3f<4>& V1,
			TSimdVec3f<4>& W1)
		{
			for (int32 LaneIndex = 0; LaneIndex < 4; ++LaneIndex)
			{
				const FSolverBody* LaneBody0 = Body0.GetValue(LaneIndex);
				const FSolverBody* LaneBody1 = Body1.GetValue(LaneIndex);
				if (LaneBody0 != nullptr)
				{
					V0.SetValue(LaneIndex, LaneBody0->V());
					W0.SetValue(LaneIndex, LaneBody0->W());
				}
				if (LaneBody1 != nullptr)
				{
					V1.SetValue(LaneIndex, LaneBody1->V());
					W1.SetValue(LaneIndex, LaneBody1->W());
				}
			}
		}

		inline void ScatterBodyVelocities(
			const TSimdVec3f<4>& V0,
			const TSimdVec3f<4>& W0,
			const TSimdVec3f<4>& V1,
			const TSimdVec3f<4>& W1,
			const TSolverBodyPtrSimd<4>& Body0,
			const TSolverBodyPtrSimd<4>& Body1)
		{
			for (int32 LaneIndex = 0; LaneIndex < 4; ++LaneIndex)
			{
				FSolverBody* LaneBody0 = Body0.GetValue(LaneIndex);
				FSolverBody* LaneBody1 = Body1.GetValue(LaneIndex);
				if (LaneBody0 != nullptr)
				{
					LaneBody0->SetV(V0.GetValue(LaneIndex));
					LaneBody0->SetW(W0.GetValue(LaneIndex));
				}
				if (LaneBody0 != nullptr)
				{
					LaneBody1->SetV(V1.GetValue(LaneIndex));
					LaneBody1->SetW(W1.GetValue(LaneIndex));
				}
			}
		}

		template<int TNumLanes>
		struct TSolverBodyPtrPairSimd
		{
			TSolverBodyPtrSimd<TNumLanes> Body0;
			TSolverBodyPtrSimd<TNumLanes> Body1;
		};

		/**
		 * @brief A SIMD row of contact points from a set of FPBDCollisionSolverSimd
		*/
		template<int TNumLanes>
		class TPBDCollisionSolverManifoldPointsSimd
		{
		public:
			using FSimdVec3f = TSimdVec3f<TNumLanes>;
			using FSimdRealf = TSimdRealf<TNumLanes>;
			using FSimdInt32 = TSimdInt32<TNumLanes>;
			using FSimdSelector = TSimdSelector<TNumLanes>;
			using FSimdSolverBodyPtr = TSolverBodyPtrSimd<TNumLanes>;

			// Whether the manifold point in each lane is set up for solving
			FSimdSelector IsValid;

			// World-space contact point relative to each particle's center of mass
			FSimdVec3f SimdRelativeContactPoint0;
			FSimdVec3f SimdRelativeContactPoint1;

			// Normal PushOut
			FSimdVec3f SimdContactNormal;
			FSimdRealf SimdContactDeltaNormal;
			FSimdRealf SimdNetPushOutNormal;
			FSimdRealf SimdContactMassNormal;
			FSimdVec3f SimdContactNormalAngular0;
			FSimdVec3f SimdContactNormalAngular1;

			// Tangential PushOut
			FSimdVec3f SimdContactTangentU;
			FSimdVec3f SimdContactTangentV;
			FSimdRealf SimdContactDeltaTangentU;
			FSimdRealf SimdContactDeltaTangentV;
			FSimdRealf SimdNetPushOutTangentU;
			FSimdRealf SimdNetPushOutTangentV;
			FSimdRealf SimdStaticFrictionRatio;
			FSimdRealf SimdContactMassTangentU;
			FSimdRealf SimdContactMassTangentV;
			FSimdVec3f SimdContactTangentUAngular0;
			FSimdVec3f SimdContactTangentVAngular0;
			FSimdVec3f SimdContactTangentUAngular1;
			FSimdVec3f SimdContactTangentVAngular1;

			// Normal Impulse
			FSimdRealf SimdContactTargetVelocityNormal;
			FSimdRealf SimdNetImpulseNormal;
			FSimdRealf SimdNetImpulseTangentU;
			FSimdRealf SimdNetImpulseTangentV;
		};

		/**
		* Holds the solver
		*/
		template<int TNumLanes>
		class TPBDCollisionSolverSimd
		{
		public:
			using FSimdVec3f = TSimdVec3f<TNumLanes>;
			using FSimdRealf = TSimdRealf<TNumLanes>;
			using FSimdInt32 = TSimdInt32<TNumLanes>;
			using FSimdSelector = TSimdSelector<TNumLanes>;
			using FSimdSolverBodyPtr = TSolverBodyPtrSimd<TNumLanes>;
			using FSimdManifoldPoint = TPBDCollisionSolverManifoldPointsSimd<TNumLanes>;

			static const int32 MaxConstrainedBodies = 2;
			static const int32 MaxPointsPerConstraint = 4;

			// Create a solver that is initialized to safe defaults
			static TPBDCollisionSolverSimd<TNumLanes> MakeInitialized()
			{
				TPBDCollisionSolverSimd<TNumLanes> Solver;
				Solver.Init();
				return Solver;
			}

			// Create a solver with no initialization
			static TPBDCollisionSolverSimd<TNumLanes> MakeUninitialized()
			{
				return TPBDCollisionSolverSimd<TNumLanes>();
			}

			// NOTE: Does not initialize any properties. See MakeInitialized
			TPBDCollisionSolverSimd() {}

			FSimdInt32 NumManifoldPoints() const
			{
				return SimdNumManifoldPoints;
			}

			void SetNumManifoldPoints(const FSimdInt32& InNum)
			{
				SimdNumManifoldPoints = InNum;
			}

			void SetManifoldPointsBuffer(const int32 InBeginIndex, const int32 InMax)
			{
				ManifoldPointBeginIndex = InBeginIndex;
				MaxManifoldPoints = InMax;
				SimdNumManifoldPoints = FSimdInt32::Zero();
			}

			int32 GetMaxManifoldPoints() const
			{
				return MaxManifoldPoints;
			}

			const FSimdManifoldPoint& GetManifoldPoint(
				const int32 ManifoldPointIndex,
				const TArrayView<const FSimdManifoldPoint>& ManifoldPointsBuffer) const
			{
				return ManifoldPointsBuffer[GetBufferIndex(ManifoldPointIndex)];
			}

			/** Reset the state of the collision solver */
			void Reset()
			{
				ManifoldPointBeginIndex = 0;
				MaxManifoldPoints = 0;
				SimdNumManifoldPoints = FSimdInt32::Zero();
			}

			void ResetManifold()
			{
				SimdNumManifoldPoints = FSimdInt32::Zero();
			}

			void SetFriction(const FSimdRealf InStaticFriction, const FSimdRealf InDynamicFriction, const FSimdRealf InVelocityFriction)
			{
				SimdStaticFriction = InStaticFriction;
				SimdDynamicFriction = InDynamicFriction;
				SimdVelocityFriction = InVelocityFriction;
			}

			void SetStiffness(const FSimdRealf InStiffness)
			{
				SimdStiffness = InStiffness;
			}

			void InitManifoldPoints(const TArrayView<FSimdManifoldPoint>& ManifoldPointsBuffer)
			{
				// Not required as long as we are zeroing the full points array on resize
				// Otherwise we need to reset all outputs and some properties here as not all lanes will be used by all points
			}

			/**
			 * Set up a manifold point ready for solving
			*/
			void SetManifoldPoint(
				const int32 ManifoldPointIndex,
				const int32 LaneIndex,
				const TArrayView<FSimdManifoldPoint>& ManifoldPointsBuffer,
				const FSolverVec3& InRelativeContactPosition0,
				const FSolverVec3& InRelativeContactPosition1,
				const FSolverVec3& InWorldContactNormal,
				const FSolverVec3& InWorldContactTangentU,
				const FSolverVec3& InWorldContactTangentV,
				const FSolverReal InWorldContactDeltaNormal,
				const FSolverReal InWorldContactDeltaTangentU,
				const FSolverReal InWorldContactDeltaTangentV,
				const FSolverReal InWorldContactVelocityTargetNormal,
				const FSolverBody& Body0,
				const FSolverBody& Body1,
				const FSolverReal InvMScale0,
				const FSolverReal InvIScale0,
				const FSolverReal InvMScale1,
				const FSolverReal InvIScale1)
			{
				const int32 BufferIndex = GetBufferIndex(ManifoldPointIndex);
				FSimdManifoldPoint& ManifoldPoint = ManifoldPointsBuffer[BufferIndex];

				ManifoldPoint.IsValid.SetValue(LaneIndex, true);

				ManifoldPoint.SimdRelativeContactPoint0.SetValue(LaneIndex, InRelativeContactPosition0);
				ManifoldPoint.SimdRelativeContactPoint1.SetValue(LaneIndex, InRelativeContactPosition1);
				ManifoldPoint.SimdContactNormal.SetValue(LaneIndex, InWorldContactNormal);
				ManifoldPoint.SimdContactTangentU.SetValue(LaneIndex, InWorldContactTangentU);
				ManifoldPoint.SimdContactTangentV.SetValue(LaneIndex, InWorldContactTangentV);
				ManifoldPoint.SimdContactDeltaNormal.SetValue(LaneIndex, InWorldContactDeltaNormal);
				ManifoldPoint.SimdContactDeltaTangentU.SetValue(LaneIndex, InWorldContactDeltaTangentU);
				ManifoldPoint.SimdContactDeltaTangentV.SetValue(LaneIndex, InWorldContactDeltaTangentV);
				ManifoldPoint.SimdContactTargetVelocityNormal.SetValue(LaneIndex, InWorldContactVelocityTargetNormal);

				UpdateManifoldPointMass(
					ManifoldPointIndex, 
					LaneIndex, 
					ManifoldPointsBuffer, 
					Body0, 
					Body1,
					InvMScale0,
					InvIScale0,
					InvMScale1,
					InvIScale1);
			}

			/**
			 * @brief Update the cached mass properties based on the current body transforms
			*/
			void UpdateManifoldPointMass(
				const int32 ManifoldPointIndex,
				const int32 LaneIndex,
				const TArrayView<FSimdManifoldPoint>& ManifoldPointsBuffer,
				const FSolverBody& Body0,
				const FSolverBody& Body1,
				const FSolverReal InvMScale0,
				const FSolverReal InvIScale0,
				const FSolverReal InvMScale1,
				const FSolverReal InvIScale1)
			{
				const int32 BufferIndex = GetBufferIndex(ManifoldPointIndex);
				FSimdManifoldPoint& ManifoldPoint = ManifoldPointsBuffer[BufferIndex];

				if (!ManifoldPoint.IsValid.GetValue(LaneIndex))
				{
					return;
				}

				FSolverReal ContactMassInvNormal = FSolverReal(0);
				FSolverReal ContactMassInvTangentU = FSolverReal(0);
				FSolverReal ContactMassInvTangentV = FSolverReal(0);

				SimdInvM0.SetValue(LaneIndex, 0);
				SimdInvM1.SetValue(LaneIndex, 0);
				ManifoldPoint.SimdContactNormalAngular0.SetValue(LaneIndex, FVec3f(0));
				ManifoldPoint.SimdContactTangentUAngular0.SetValue(LaneIndex, FVec3f(0));
				ManifoldPoint.SimdContactTangentVAngular0.SetValue(LaneIndex, FVec3f(0));
				ManifoldPoint.SimdContactNormalAngular1.SetValue(LaneIndex, FVec3f(0));
				ManifoldPoint.SimdContactTangentUAngular1.SetValue(LaneIndex, FVec3f(0));
				ManifoldPoint.SimdContactTangentVAngular1.SetValue(LaneIndex, FVec3f(0));

				const FVec3f ContactNormal = ManifoldPoint.SimdContactNormal.GetValue(LaneIndex);
				const FVec3f ContactTangentU = ManifoldPoint.SimdContactTangentU.GetValue(LaneIndex);
				const FVec3f ContactTangentV = ManifoldPoint.SimdContactTangentV.GetValue(LaneIndex);

				const FSolverReal InvM0 = InvMScale0 * Body0.InvM();
				const FSolverReal InvM1 = InvMScale1 * Body1.InvM();
				if (InvM0 > 0)
				{
					const FSolverMatrix33 InvI0 = InvIScale0 * Body0.InvI();

					const FVec3f RelativeContactPoint0 = ManifoldPoint.SimdRelativeContactPoint0.GetValue(LaneIndex);
					const FSolverVec3 R0xN = FSolverVec3::CrossProduct(RelativeContactPoint0, ContactNormal);
					const FSolverVec3 R0xU = FSolverVec3::CrossProduct(RelativeContactPoint0, ContactTangentU);
					const FSolverVec3 R0xV = FSolverVec3::CrossProduct(RelativeContactPoint0, ContactTangentV);
					const FSolverVec3 IR0xN = InvI0 * R0xN;
					const FSolverVec3 IR0xU = InvI0 * R0xU;
					const FSolverVec3 IR0xV = InvI0 * R0xV;

					SimdInvM0.SetValue(LaneIndex, InvM0);

					ManifoldPoint.SimdContactNormalAngular0.SetValue(LaneIndex, IR0xN);
					ManifoldPoint.SimdContactTangentUAngular0.SetValue(LaneIndex, IR0xU);
					ManifoldPoint.SimdContactTangentVAngular0.SetValue(LaneIndex, IR0xV);

					ContactMassInvNormal += FSolverVec3::DotProduct(R0xN, IR0xN) + InvM0;
					ContactMassInvTangentU += FSolverVec3::DotProduct(R0xU, IR0xU) + InvM0;
					ContactMassInvTangentV += FSolverVec3::DotProduct(R0xV, IR0xV) + InvM0;
				}
				if (InvM1 > 0)
				{
					const FSolverMatrix33 InvI1 = InvIScale1 * Body1.InvI();

					const FVec3f RelativeContactPoint1 = ManifoldPoint.SimdRelativeContactPoint1.GetValue(LaneIndex);
					const FSolverVec3 R1xN = FSolverVec3::CrossProduct(RelativeContactPoint1, ContactNormal);
					const FSolverVec3 R1xU = FSolverVec3::CrossProduct(RelativeContactPoint1, ContactTangentU);
					const FSolverVec3 R1xV = FSolverVec3::CrossProduct(RelativeContactPoint1, ContactTangentV);
					const FSolverVec3 IR1xN = InvI1 * R1xN;
					const FSolverVec3 IR1xU = InvI1 * R1xU;
					const FSolverVec3 IR1xV = InvI1 * R1xV;

					SimdInvM1.SetValue(LaneIndex, InvM1);

					ManifoldPoint.SimdContactNormalAngular1.SetValue(LaneIndex, IR1xN);
					ManifoldPoint.SimdContactTangentUAngular1.SetValue(LaneIndex, IR1xU);
					ManifoldPoint.SimdContactTangentVAngular1.SetValue(LaneIndex, IR1xV);

					ContactMassInvNormal += FSolverVec3::DotProduct(R1xN, IR1xN) + InvM1;
					ContactMassInvTangentU += FSolverVec3::DotProduct(R1xU, IR1xU) + InvM1;
					ContactMassInvTangentV += FSolverVec3::DotProduct(R1xV, IR1xV) + InvM1;
				}

				ManifoldPoint.SimdContactMassNormal.SetValue(LaneIndex, (ContactMassInvNormal > FSolverReal(UE_SMALL_NUMBER)) ? FSolverReal(1) / ContactMassInvNormal : FSolverReal(0));
				ManifoldPoint.SimdContactMassTangentU.SetValue(LaneIndex, (ContactMassInvTangentU > FSolverReal(UE_SMALL_NUMBER)) ? FSolverReal(1) / ContactMassInvTangentU : FSolverReal(0));
				ManifoldPoint.SimdContactMassTangentV.SetValue(LaneIndex, (ContactMassInvTangentV > FSolverReal(UE_SMALL_NUMBER)) ? FSolverReal(1) / ContactMassInvTangentV : FSolverReal(0));
			}

			void UpdateManifoldPointMassNormal(
				const int32 ManifoldPointIndex,
				const int32 LaneIndex,
				const TArrayView<FSimdManifoldPoint>& ManifoldPointsBuffer,
				const FSolverBody& Body0,
				const FSolverBody& Body1,
				const FSolverReal InvMScale0,
				const FSolverReal InvIScale0,
				const FSolverReal InvMScale1,
				const FSolverReal InvIScale1)
			{
				const int32 BufferIndex = GetBufferIndex(ManifoldPointIndex);
				FSimdManifoldPoint& ManifoldPoint = ManifoldPointsBuffer[BufferIndex];

				if (!ManifoldPoint.IsValid.GetValue(LaneIndex))
				{
					return;
				}

				FSolverReal ContactMassInvNormal = FSolverReal(0);

				SimdInvM0.SetValue(LaneIndex, 0);
				SimdInvM1.SetValue(LaneIndex, 0);
				ManifoldPoint.SimdContactNormalAngular0.SetValue(LaneIndex, FVec3f(0));
				ManifoldPoint.SimdContactNormalAngular1.SetValue(LaneIndex, FVec3f(0));

				const FVec3f ContactNormal = ManifoldPoint.SimdContactNormal.GetValue(LaneIndex);

				const FSolverReal InvM0 = InvMScale0 * Body0.InvM();
				const FSolverReal InvM1 = InvMScale1 * Body1.InvM();
				if (InvM0 > 0)
				{
					const FSolverMatrix33 InvI0 = InvIScale0 * Body0.InvI();

					const FVec3f RelativeContactPoint0 = ManifoldPoint.SimdRelativeContactPoint0.GetValue(LaneIndex);
					const FSolverVec3 R0xN = FSolverVec3::CrossProduct(RelativeContactPoint0, ContactNormal);
					const FSolverVec3 IR0xN = InvI0 * R0xN;

					SimdInvM0.SetValue(LaneIndex, InvM0);

					ManifoldPoint.SimdContactNormalAngular0.SetValue(LaneIndex, IR0xN);

					ContactMassInvNormal += FSolverVec3::DotProduct(R0xN, IR0xN) + InvM0;
				}
				if (InvM1 > 0)
				{
					const FSolverMatrix33 InvI1 = InvIScale1 * Body1.InvI();

					const FVec3f RelativeContactPoint1 = ManifoldPoint.SimdRelativeContactPoint1.GetValue(LaneIndex);
					const FSolverVec3 R1xN = FSolverVec3::CrossProduct(RelativeContactPoint1, ContactNormal);
					const FSolverVec3 IR1xN = InvI1 * R1xN;

					SimdInvM1.SetValue(LaneIndex, InvM1);

					ManifoldPoint.SimdContactNormalAngular1.SetValue(LaneIndex, IR1xN);

					ContactMassInvNormal += FSolverVec3::DotProduct(R1xN, IR1xN) + InvM1;
				}

				const FSolverReal ContactMassNormal = (ContactMassInvNormal > FSolverReal(UE_SMALL_NUMBER)) ? FSolverReal(1) / ContactMassInvNormal : FSolverReal(0);
				ManifoldPoint.SimdContactMassNormal.SetValue(LaneIndex, ContactMassNormal);
			}

			void UpdateMassNormal(
				const int32 LaneIndex,
				const TArrayView<FSimdManifoldPoint>& ManifoldPointsBuffer,
				const FSolverBody& Body0,
				const FSolverBody& Body1,
				const FSolverReal InvMScale0,
				const FSolverReal InvIScale0,
				const FSolverReal InvMScale1,
				const FSolverReal InvIScale1)
			{
				const int32 NumManifoldPoints = SimdNumManifoldPoints.GetValue(LaneIndex);
				for (int32 ManifoldPointIndex = 0; ManifoldPointIndex < NumManifoldPoints; ++ManifoldPointIndex)
				{
					UpdateManifoldPointMassNormal(
						ManifoldPointIndex,
						LaneIndex,
						ManifoldPointsBuffer,
						Body0,
						Body1,
						InvMScale0,
						InvIScale0,
						InvMScale1,
						InvIScale1);
				}
			}

			void SolvePositionNoFriction(
				const TArrayView<FSimdManifoldPoint>& ManifoldPointsBuffer,
				const FSimdSolverBodyPtr& Body0,
				const FSimdSolverBodyPtr& Body1,
				const FSimdRealf& MaxPushOut)
			{
				// Get the current corrections for each body. 
				// NOTE: This is a gather operation
				FSimdVec3f DP0, DQ0, DP1, DQ1;
				GatherBodyPositionCorrections(Body0, Body1, DP0, DQ0, DP1, DQ1);

				for (int32 ManifoldPointIndex = 0; ManifoldPointIndex < MaxManifoldPoints; ++ManifoldPointIndex)
				{
					FSimdManifoldPoint& ManifoldPoint = ManifoldPointsBuffer[GetBufferIndex(ManifoldPointIndex)];

					// Which lanes require this point be simulated?
					//if (!SimdAnyTrue(ManifoldPoint.IsValid))
					//{
					//	continue;
					//}

					// Calculate the contact error
					const FSimdVec3f DQ0xR0 = SimdCrossProduct(DQ0, ManifoldPoint.SimdRelativeContactPoint0);
					const FSimdVec3f DQ1xR1 = SimdCrossProduct(DQ1, ManifoldPoint.SimdRelativeContactPoint1);
					const FSimdVec3f ContactDelta0 = SimdAdd(DP0, DQ0xR0);
					const FSimdVec3f ContactDelta1 = SimdAdd(DP1, DQ1xR1);
					const FSimdVec3f ContactDelta = SimdSubtract(ContactDelta0, ContactDelta1);
					FSimdRealf ContactErrorNormal = SimdAdd(ManifoldPoint.SimdContactDeltaNormal, SimdDotProduct(ContactDelta, ManifoldPoint.SimdContactNormal));

					// Apply MaxPushOut clamping if required
					//if ((MaxPushOut > 0) && (ContactErrorNormal < -MaxPushOut)) { ContactErrorNormal = -MaxPushOut; }
					const FSimdRealf NegMaxPushOut = SimdNegate(MaxPushOut);
					const FSimdSelector ShouldClampError = SimdAnd(SimdLess(NegMaxPushOut, FSimdRealf::Zero()), SimdLess(ContactErrorNormal, NegMaxPushOut));
					ContactErrorNormal = SimdSelect(ShouldClampError, NegMaxPushOut, ContactErrorNormal);

					// Determine which lanes to process: only those with an overlap or that applied a pushout on a prior iteration
					const FSimdSelector IsErrorNegative = SimdLess(ContactErrorNormal, FSimdRealf::Zero());
					const FSimdSelector IsNetPushOutPositive = SimdGreater(ManifoldPoint.SimdNetPushOutNormal, FSimdRealf::Zero());
					const FSimdSelector ShouldProcess = SimdAnd(ManifoldPoint.IsValid, SimdOr(IsErrorNegative, IsNetPushOutPositive));

					// If all lanes are to be skipped, early-out
					//if (!SimdAnyTrue(ShouldProcess))
					//{
					//	continue;
					//}

					// Zero out the error for points we should not process so we don't apply a correction for them
					ContactErrorNormal = SimdSelect(ShouldProcess, ContactErrorNormal, FSimdRealf::Zero());

					FSimdRealf PushOutNormal = SimdNegate(SimdMultiply(ContactErrorNormal, SimdMultiply(SimdStiffness, ManifoldPoint.SimdContactMassNormal)));

					// Unilateral constraint: Net-negative pushout is not allowed, but
					// PushOutNormal may be negative on any iteration as long as the net is positive
					// If the net goes negative, apply a pushout to make it zero
					const FSimdSelector IsPositive = SimdGreater(SimdAdd(ManifoldPoint.SimdNetPushOutNormal, PushOutNormal), FSimdRealf::Zero());
					PushOutNormal = SimdSelect(IsPositive, PushOutNormal, SimdNegate(ManifoldPoint.SimdNetPushOutNormal));

					// New net pushout
					ManifoldPoint.SimdNetPushOutNormal = SimdAdd(PushOutNormal, ManifoldPoint.SimdNetPushOutNormal);

					// Convert the positional impulse into position and rotation corrections for each body
					// NOTE: order of operations matches FPBDCollisionSolver::ApplyPositionCorrectionNormal so we can AB test
					const FSimdVec3f DDP0 = SimdMultiply(ManifoldPoint.SimdContactNormal, SimdMultiply(SimdInvM0, PushOutNormal));
					const FSimdVec3f DDQ0 = SimdMultiply(ManifoldPoint.SimdContactNormalAngular0, PushOutNormal);
					const FSimdVec3f DDP1 = SimdMultiply(ManifoldPoint.SimdContactNormal, SimdMultiply(SimdInvM1, PushOutNormal));
					const FSimdVec3f DDQ1 = SimdMultiply(ManifoldPoint.SimdContactNormalAngular1, PushOutNormal);
					DP0 = SimdAdd(DP0, DDP0);
					DQ0 = SimdAdd(DQ0, DDQ0);
					DP1 = SimdSubtract(DP1, DDP1);
					DQ1 = SimdSubtract(DQ1, DDQ1);
				}

				// Update the corrections on the bodies. 
				// NOTE: This is a scatter operation
				ScatterBodyPositionCorrections(DP0, DQ0, DP1, DQ1, Body0, Body1);
			}

			void SolvePositionWithFriction(
				const TArrayView<FSimdManifoldPoint>& ManifoldPointsBuffer,
				const FSimdSolverBodyPtr& Body0,
				const FSimdSolverBodyPtr& Body1,
				const FSimd4Realf& MaxPushOut,
				const FSimd4Realf& FrictionStiffnessScale)
			{
				// Get the current corrections for each body. 
				// NOTE: This is a gather operation
				FSimdVec3f DP0, DQ0, DP1, DQ1;
				GatherBodyPositionCorrections(Body0, Body1, DP0, DQ0, DP1, DQ1);

				const FSimdRealf Zero = FSimdRealf::Zero();
				const FSimdRealf One = FSimdRealf::One();

				FSimdRealf NumFrictionPoints = Zero;
				FSimdRealf MaxFrictionPushOut = Zero;

				// @todo(chaos): can these be zero (and make non-simd version match)?
				const FSimdRealf PushOutNormalTolerance = FSimdRealf::Make(UE_SMALL_NUMBER);
				const FSimdRealf MaxFrictionPushOutTolerance = FSimdRealf::Make(UE_KINDA_SMALL_NUMBER);

				// Apply the normal pushout and calculate the net normal pushout for the friction limit
				for (int32 ManifoldPointIndex = 0; ManifoldPointIndex < MaxManifoldPoints; ++ManifoldPointIndex)
				{
					FSimdManifoldPoint& ManifoldPoint = ManifoldPointsBuffer[GetBufferIndex(ManifoldPointIndex)];

					// Which lanes require this point be simulated?
					//if (!SimdAnyTrue(ManifoldPoint.IsValid))
					//{
					//	continue;
					//}

					// Calculate the contact error
					const FSimdVec3f DQ0xR0 = SimdCrossProduct(DQ0, ManifoldPoint.SimdRelativeContactPoint0);
					const FSimdVec3f DQ1xR1 = SimdCrossProduct(DQ1, ManifoldPoint.SimdRelativeContactPoint1);
					const FSimdVec3f ContactDelta0 = SimdAdd(DP0, DQ0xR0);
					const FSimdVec3f ContactDelta1 = SimdAdd(DP1, DQ1xR1);
					const FSimdVec3f ContactDelta = SimdSubtract(ContactDelta0, ContactDelta1);
					FSimdRealf ContactErrorNormal = SimdAdd(ManifoldPoint.SimdContactDeltaNormal, SimdDotProduct(ContactDelta, ManifoldPoint.SimdContactNormal));

					// Apply MaxPushOut clamping if required
					//if ((MaxPushOut > 0) && (ContactErrorNormal < -MaxPushOut)) { ContactErrorNormal = -MaxPushOut; }
					const FSimdRealf NegMaxPushOut = SimdNegate(MaxPushOut);
					const FSimdSelector ShouldClampError = SimdAnd(SimdLess(NegMaxPushOut, Zero), SimdLess(ContactErrorNormal, NegMaxPushOut));
					ContactErrorNormal = SimdSelect(ShouldClampError, NegMaxPushOut, ContactErrorNormal);

					// Determine which lanes to process: only those with an overlap or that applied a pushout on a prior iteration
					const FSimdSelector IsErrorNegative = SimdLess(ContactErrorNormal, Zero);
					const FSimdSelector IsNetPushOutPositive = SimdGreater(ManifoldPoint.SimdNetPushOutNormal, PushOutNormalTolerance);
					const FSimdSelector ProcessManifoldPoint = SimdAnd(ManifoldPoint.IsValid, SimdOr(IsErrorNegative, IsNetPushOutPositive));

					// If all lanes are to be skipped, early-out
					//if (!SimdAnyTrue(ProcessManifoldPoint))
					//{
					//	continue;
					//}

					// Zero out the error for points we should not process so we don't apply a correction for them
					ContactErrorNormal = SimdSelect(ProcessManifoldPoint, ContactErrorNormal, Zero);

					FSimdRealf PushOutNormal = SimdNegate(SimdMultiply(ContactErrorNormal, SimdMultiply(SimdStiffness, ManifoldPoint.SimdContactMassNormal)));

					// Unilateral constraint: Net-negative pushout is not allowed, but
					// PushOutNormal may be negative on any iteration as long as the net is positive
					// If the net goes negative, apply a pushout to make it zero
					const FSimdSelector IsPositive = SimdGreater(SimdAdd(ManifoldPoint.SimdNetPushOutNormal, PushOutNormal), Zero);
					PushOutNormal = SimdSelect(IsPositive, PushOutNormal, SimdNegate(ManifoldPoint.SimdNetPushOutNormal));

					// New net pushout
					ManifoldPoint.SimdNetPushOutNormal = SimdAdd(PushOutNormal, ManifoldPoint.SimdNetPushOutNormal);

					// Convert the positional impulse into position and rotation corrections for each body
					// NOTE: order of operations matches FPBDCollisionSolver::ApplyPositionCorrectionNormal so we can AB test
					const FSimdVec3f DDP0 = SimdMultiply(ManifoldPoint.SimdContactNormal, SimdMultiply(SimdInvM0, PushOutNormal));
					const FSimdVec3f DDQ0 = SimdMultiply(ManifoldPoint.SimdContactNormalAngular0, PushOutNormal);
					const FSimdVec3f DDP1 = SimdMultiply(ManifoldPoint.SimdContactNormal, SimdMultiply(SimdInvM1, PushOutNormal));
					const FSimdVec3f DDQ1 = SimdMultiply(ManifoldPoint.SimdContactNormalAngular1, PushOutNormal);
					DP0 = SimdAdd(DP0, DDP0);
					DQ0 = SimdAdd(DQ0, DDQ0);
					DP1 = SimdSubtract(DP1, DDP1);
					DQ1 = SimdSubtract(DQ1, DDQ1);

					MaxFrictionPushOut = SimdSelect(IsPositive, SimdAdd(MaxFrictionPushOut, ManifoldPoint.SimdNetPushOutNormal), MaxFrictionPushOut);
					NumFrictionPoints = SimdSelect(IsPositive, SimdAdd(NumFrictionPoints, One), NumFrictionPoints);
				}

				MaxFrictionPushOut = SimdSelect(SimdGreater(NumFrictionPoints, Zero), SimdDivide(MaxFrictionPushOut, NumFrictionPoints), Zero);

				for (int32 ManifoldPointIndex = 0; ManifoldPointIndex < MaxManifoldPoints; ++ManifoldPointIndex)
				{
					FSimdManifoldPoint& ManifoldPoint = ManifoldPointsBuffer[GetBufferIndex(ManifoldPointIndex)];

					// Which lanes require this point be simulated?
					//if (!SimdAnyTrue(ManifoldPoint.IsValid))
					//{
					//	continue;
					//}

					// Calculate the contact error
					const FSimdVec3f DQ0xR0 = SimdCrossProduct(DQ0, ManifoldPoint.SimdRelativeContactPoint0);
					const FSimdVec3f DQ1xR1 = SimdCrossProduct(DQ1, ManifoldPoint.SimdRelativeContactPoint1);
					const FSimdVec3f ContactDelta0 = SimdAdd(DP0, DQ0xR0);
					const FSimdVec3f ContactDelta1 = SimdAdd(DP1, DQ1xR1);
					const FSimdVec3f ContactDelta = SimdSubtract(ContactDelta0, ContactDelta1);

					// Should we apply tangential corrections for friction? 
					// bUpdateFriction = ((TotalPushOutNormal > 0) || (NetPushOutTangentU != 0) || (NetPushOutTangentV != 0))
					const FSimdSelector HasFriction = SimdGreater(SimdStaticFriction, Zero);
					const FSimdSelector HasNormalPushout = SimdGreater(ManifoldPoint.SimdNetPushOutNormal, Zero);
					const FSimdSelector HasTangentUPushout = SimdNotEqual(ManifoldPoint.SimdNetPushOutTangentU, Zero);
					const FSimdSelector HasTangentVPushout = SimdNotEqual(ManifoldPoint.SimdNetPushOutTangentV, Zero);
					const FSimdSelector ApplyPointFriction = SimdAnd(HasFriction, SimdOr(HasNormalPushout, SimdOr(HasTangentUPushout, HasTangentVPushout)));
					//if (SimdAnyTrue(ApplyPointFriction))
					{
						// Calculate tangential errors
						const FSimdRealf ContactErrorTangentU = SimdAdd(ManifoldPoint.SimdContactDeltaTangentU, SimdDotProduct(ContactDelta, ManifoldPoint.SimdContactTangentU));
						const FSimdRealf ContactErrorTangentV = SimdAdd(ManifoldPoint.SimdContactDeltaTangentV, SimdDotProduct(ContactDelta, ManifoldPoint.SimdContactTangentV));

						// A stiffness multiplier on the impulses
						const FSimdRealf FrictionStiffness = SimdMultiply(FrictionStiffnessScale, SimdStiffness);

						// Calculate tangential correction
						FSimdRealf PushOutTangentU = SimdNegate(SimdMultiply(FrictionStiffness, SimdMultiply(ContactErrorTangentU, ManifoldPoint.SimdContactMassTangentU)));
						FSimdRealf PushOutTangentV = SimdNegate(SimdMultiply(FrictionStiffness, SimdMultiply(ContactErrorTangentV, ManifoldPoint.SimdContactMassTangentV)));

						// New net tangential pushouts
						FSimdRealf NetPushOutTangentU = SimdAdd(ManifoldPoint.SimdNetPushOutTangentU, PushOutTangentU);
						FSimdRealf NetPushOutTangentV = SimdAdd(ManifoldPoint.SimdNetPushOutTangentV, PushOutTangentV);
						FSimdRealf StaticFrictionRatio = One;

						// Should we clamp to the friction cone or reset the friction?
						const FSimdSelector ApplyFrictionCone = SimdGreaterEqual(MaxFrictionPushOut, MaxFrictionPushOutTolerance);

						// Apply cone limits to tangential pushouts on lanes that exceed the limit
						// NOTE: if HasNormalPushout is false in any lane, we have already zeroed the net pushout and don't need to do anything here
						//if (SimdAnyTrue(ApplyFrictionCone))
						{
							const FSimdRealf MaxStaticPushOutTangentSq = SimdSquare(SimdMultiply(SimdStaticFriction, MaxFrictionPushOut));
							const FSimdRealf NetPushOutTangentSq = SimdAdd(SimdSquare(NetPushOutTangentU), SimdSquare(NetPushOutTangentV));
							const FSimdSelector ExceededFrictionCone = SimdAnd(ApplyFrictionCone, SimdGreater(NetPushOutTangentSq, MaxStaticPushOutTangentSq));
							//if (SimdAnyTrue(ExceededFrictionCone))
							{
								const FSimdRealf MaxDynamicPushOutTangent = SimdMultiply(SimdDynamicFriction, MaxFrictionPushOut);
								const FSimdRealf FrictionMultiplier = SimdMultiply(MaxDynamicPushOutTangent, SimdInvSqrt(NetPushOutTangentSq));
								const FSimdRealf ClampedNetPushOutTangentU = SimdMultiply(FrictionMultiplier, NetPushOutTangentU);
								const FSimdRealf ClampedNetPushOutTangentV = SimdMultiply(FrictionMultiplier, NetPushOutTangentV);
								const FSimdRealf ClampedPushOutTangentU = SimdSubtract(ClampedNetPushOutTangentU, ManifoldPoint.SimdNetPushOutTangentU);
								const FSimdRealf ClampedPushOutTangentV = SimdSubtract(ClampedNetPushOutTangentV, ManifoldPoint.SimdNetPushOutTangentV);

								PushOutTangentU = SimdSelect(ExceededFrictionCone, ClampedPushOutTangentU, PushOutTangentU);
								PushOutTangentV = SimdSelect(ExceededFrictionCone, ClampedPushOutTangentV, PushOutTangentV);
								NetPushOutTangentU = SimdSelect(ExceededFrictionCone, ClampedNetPushOutTangentU, NetPushOutTangentU);
								NetPushOutTangentV = SimdSelect(ExceededFrictionCone, ClampedNetPushOutTangentV, NetPushOutTangentV);
								StaticFrictionRatio = SimdSelect(ExceededFrictionCone, FrictionMultiplier, StaticFrictionRatio);
							}
						}

						// If we did not apply the friction cone because we have no normal impulse, apply a pushout to cancel previously applied friction
						//if (!SimdAllTrue(ApplyFrictionCone))
						{
							PushOutTangentU = SimdSelect(ApplyFrictionCone, PushOutTangentU, SimdNegate(ManifoldPoint.SimdNetPushOutTangentU));
							PushOutTangentV = SimdSelect(ApplyFrictionCone, PushOutTangentV, SimdNegate(ManifoldPoint.SimdNetPushOutTangentV));
							NetPushOutTangentU = SimdSelect(ApplyFrictionCone, NetPushOutTangentU, Zero);
							NetPushOutTangentV = SimdSelect(ApplyFrictionCone, NetPushOutTangentV, Zero);
							StaticFrictionRatio = SimdSelect(ApplyFrictionCone, StaticFrictionRatio, Zero);
						}

						// Undo all our good work for lanes that should not apply friction at all
						//if (!SimdAllTrue(ApplyPointFriction))
						{
							PushOutTangentU = SimdSelect(ApplyPointFriction, PushOutTangentU, Zero);
							PushOutTangentV = SimdSelect(ApplyPointFriction, PushOutTangentV, Zero);
							NetPushOutTangentU = SimdSelect(ApplyPointFriction, NetPushOutTangentU, ManifoldPoint.SimdNetPushOutTangentU);
							NetPushOutTangentV = SimdSelect(ApplyPointFriction, NetPushOutTangentV, ManifoldPoint.SimdNetPushOutTangentV);
							StaticFrictionRatio = SimdSelect(ApplyPointFriction, StaticFrictionRatio, One);
						}

						ManifoldPoint.SimdNetPushOutTangentU = NetPushOutTangentU;
						ManifoldPoint.SimdNetPushOutTangentV = NetPushOutTangentV;
						ManifoldPoint.SimdStaticFrictionRatio = StaticFrictionRatio;

						// Add the tangential corrections to the applied correction
						// NOTE: The order of operations here matches FPBDCollisionSolver::ApplyPositionCorrectionTangential
						// so that we can do an AB test of the SIMD solver versus the standard solver
						const FSimdVec3f PushOut = SimdAdd(SimdMultiply(PushOutTangentU, ManifoldPoint.SimdContactTangentU), SimdMultiply(PushOutTangentV, ManifoldPoint.SimdContactTangentV));
						const FSimdVec3f DDP0 = SimdMultiply(SimdInvM0, PushOut);
						const FSimdVec3f DDP1 = SimdMultiply(SimdInvM1, PushOut);
						const FSimdVec3f DDQ0 = SimdAdd(SimdMultiply(ManifoldPoint.SimdContactTangentUAngular0, PushOutTangentU), SimdMultiply(ManifoldPoint.SimdContactTangentVAngular0, PushOutTangentV));
						const FSimdVec3f DDQ1 = SimdAdd(SimdMultiply(ManifoldPoint.SimdContactTangentUAngular1, PushOutTangentU), SimdMultiply(ManifoldPoint.SimdContactTangentVAngular1, PushOutTangentV));
						DP0 = SimdAdd(DP0, DDP0);
						DQ0 = SimdAdd(DQ0, DDQ0);
						DP1 = SimdSubtract(DP1, DDP1);
						DQ1 = SimdSubtract(DQ1, DDQ1);
					}
				}

				// Update the corrections on the bodies. 
				// NOTE: This is a scatter operation
				ScatterBodyPositionCorrections(DP0, DQ0, DP1, DQ1, Body0, Body1);
			}

			void SolveVelocityNoFriction(
				const TArrayView<FSimdManifoldPoint>& ManifoldPointsBuffer,
				const FSimdSolverBodyPtr& Body0,
				const FSimdSolverBodyPtr& Body1,
				const FSimdRealf& Dt)
			{
				// Gather the body data we need
				FSimdVec3f V0, W0, V1, W1;
				GatherBodyVelocities(Body0, Body1, V0, W0, V1, W1);

				for (int32 ManifoldPointIndex = 0; ManifoldPointIndex < MaxManifoldPoints; ++ManifoldPointIndex)
				{
					FSimdManifoldPoint& ManifoldPoint = ManifoldPointsBuffer[GetBufferIndex(ManifoldPointIndex)];

					// Which lanes require this point be simulated?
					//if (!SimdAnyTrue(ManifoldPoint.IsValid))
					//{
					//	continue;
					//}

					// Only lanes that applied a position correction or that started with an overlap require a velocity correction
					FSimdSelector ShouldSolveVelocity = SimdOr(SimdGreater(ManifoldPoint.SimdNetPushOutNormal, FSimdRealf::Zero()), SimdLess(ManifoldPoint.SimdContactDeltaNormal, FSimdRealf::Zero()));
					ShouldSolveVelocity = SimdAnd(ManifoldPoint.IsValid, ShouldSolveVelocity);
					//if (!SimdAnyTrue(ShouldSolveVelocity))
					//{
					//	continue;
					//}

					// Calculate the velocity error we need to correct
					const FSimdVec3f ContactVelocity0 = SimdAdd(V0, SimdCrossProduct(W0, ManifoldPoint.SimdRelativeContactPoint0));
					const FSimdVec3f ContactVelocity1 = SimdAdd(V1, SimdCrossProduct(W1, ManifoldPoint.SimdRelativeContactPoint1));
					const FSimdVec3f ContactVelocity = SimdSubtract(ContactVelocity0, ContactVelocity1);
					const FSimdRealf ContactVelocityNormal = SimdDotProduct(ContactVelocity, ManifoldPoint.SimdContactNormal);
					FSimdRealf ContactVelocityErrorNormal = SimdSubtract(ContactVelocityNormal, ManifoldPoint.SimdContactTargetVelocityNormal);

					// Calculate the velocity correction for the error
					FSimdRealf ImpulseNormal = SimdNegate(SimdMultiply(SimdMultiply(SimdStiffness, ManifoldPoint.SimdContactMassNormal), ContactVelocityErrorNormal));

					// The minimum normal impulse we can apply. We are allowed to apply a negative impulse 
					// up to an amount that would conteract the implciit velocity applied by the pushout
					// MinImpulseNormal = FMath::Min(0, -NetPushOutNormal / Dt)
					// if (NetImpulseNormal + ImpulseNormal < MinImpulseNormal) {...}
					const FSimdRealf MinImpulseNormal = SimdMin(SimdDivide(SimdNegate(ManifoldPoint.SimdNetPushOutNormal), Dt), FSimdRealf::Zero());
					const FSimdSelector ShouldClampImpulse = SimdLess(SimdAdd(ManifoldPoint.SimdNetImpulseNormal, ImpulseNormal), MinImpulseNormal);
					const FSimdRealf ClampedImpulseNormal = SimdSubtract(MinImpulseNormal, ManifoldPoint.SimdNetImpulseNormal);
					ImpulseNormal = SimdSelect(ShouldClampImpulse, ClampedImpulseNormal, ImpulseNormal);

					// Clear the impulse for lanes that should not be solving for velocity
					ImpulseNormal = SimdSelect(ShouldSolveVelocity, ImpulseNormal, FSimdRealf::Zero());

					ManifoldPoint.SimdNetImpulseNormal = SimdAdd(ManifoldPoint.SimdNetImpulseNormal, ImpulseNormal);

					// NOTE: order of operations matches FPBDCollisionSolver::ApplyVelocityCorrectionNormal for AB Testing
					const FSimdVec3f Impulse = SimdMultiply(ImpulseNormal, ManifoldPoint.SimdContactNormal);
					const FSimdVec3f DV0 = SimdMultiply(Impulse, SimdInvM0);
					const FSimdVec3f DV1 = SimdMultiply(Impulse, SimdInvM1);
					const FSimdVec3f DW0 = SimdMultiply(ImpulseNormal, ManifoldPoint.SimdContactNormalAngular0);
					const FSimdVec3f DW1 = SimdMultiply(ImpulseNormal, ManifoldPoint.SimdContactNormalAngular1);
					V0 = SimdAdd(V0, DV0);
					W0 = SimdAdd(W0, DW0);
					V1 = SimdSubtract(V1, DV1);
					W1 = SimdSubtract(W1, DW1);
				}

				ScatterBodyVelocities(V0, W0, V1, W1, Body0, Body1);
			}

			void SolveVelocityWithFrictionImpl(
				const TArrayView<FSimdManifoldPoint>& ManifoldPointsBuffer,
				const FSimdSolverBodyPtr& Body0,
				const FSimdSolverBodyPtr& Body1,
				const FSimdRealf& Dt,
				const FSimd4Realf& FrictionStiffnessScale)
			{
				// Gather the body data we need
				FSimdVec3f V0, W0, V1, W1;
				GatherBodyVelocities(Body0, Body1, V0, W0, V1, W1);

				for (int32 ManifoldPointIndex = 0; ManifoldPointIndex < MaxManifoldPoints; ++ManifoldPointIndex)
				{
					FSimdManifoldPoint& ManifoldPoint = ManifoldPointsBuffer[GetBufferIndex(ManifoldPointIndex)];

					// Which lanes require this point be simulated?
					//if (!SimdAnyTrue(ManifoldPoint.IsValid))
					//{
					//	continue;
					//}

					// Only lanes that applied a position correction or that started with an overlap require a velocity correction
					FSimdSelector ShouldSolveVelocity = SimdOr(SimdGreater(ManifoldPoint.SimdNetPushOutNormal, FSimdRealf::Zero()), SimdLess(ManifoldPoint.SimdContactDeltaNormal, FSimdRealf::Zero()));
					ShouldSolveVelocity = SimdAnd(ManifoldPoint.IsValid, ShouldSolveVelocity);
					//if (!SimdAnyTrue(ShouldSolveVelocity))
					//{
					//	continue;
					//}

					// Calculate the velocity error we need to correct
					const FSimdVec3f ContactVelocity0 = SimdAdd(V0, SimdCrossProduct(W0, ManifoldPoint.SimdRelativeContactPoint0));
					const FSimdVec3f ContactVelocity1 = SimdAdd(V1, SimdCrossProduct(W1, ManifoldPoint.SimdRelativeContactPoint1));
					const FSimdVec3f ContactVelocity = SimdSubtract(ContactVelocity0, ContactVelocity1);
					FSimdRealf ContactVelocityErrorNormal = SimdSubtract(SimdDotProduct(ContactVelocity, ManifoldPoint.SimdContactNormal), ManifoldPoint.SimdContactTargetVelocityNormal);
					FSimdRealf ContactVelocityErrorTangentU = SimdDotProduct(ContactVelocity, ManifoldPoint.SimdContactTangentU);
					FSimdRealf ContactVelocityErrorTangentV = SimdDotProduct(ContactVelocity, ManifoldPoint.SimdContactTangentV);

					// A stiffness multiplier on the impulses (zeroed if we have no friction in a lane)
					const FSimdRealf FrictionStiffness = SimdMultiply(FrictionStiffnessScale, SimdStiffness);

					// Calculate the impulses
					FSimdRealf ImpulseNormal = SimdNegate(SimdMultiply(SimdMultiply(SimdStiffness, ManifoldPoint.SimdContactMassNormal), ContactVelocityErrorNormal));
					FSimdRealf ImpulseTangentU = SimdNegate(SimdMultiply(SimdMultiply(FrictionStiffness, ManifoldPoint.SimdContactMassTangentU), ContactVelocityErrorTangentU));
					FSimdRealf ImpulseTangentV = SimdNegate(SimdMultiply(SimdMultiply(FrictionStiffness, ManifoldPoint.SimdContactMassTangentV), ContactVelocityErrorTangentV));

					// Zero out friction for lanes with no friction
					const FSimdSelector HasFriction = SimdAnd(ShouldSolveVelocity, SimdGreater(SimdVelocityFriction, FSimdRealf::Zero()));
					ImpulseTangentU = SimdSelect(HasFriction, ImpulseTangentU, FSimdRealf::Zero());
					ImpulseTangentV = SimdSelect(HasFriction, ImpulseTangentV, FSimdRealf::Zero());

					// The minimum normal impulse we can apply. We are allowed to apply a negative impulse 
					// up to an amount that would conteract the implciit velocity applied by the pushout
					// MinImpulseNormal = FMath::Min(0, -NetPushOutNormal / Dt)
					// if (NetImpulseNormal + ImpulseNormal < MinImpulseNormal) {...}
					const FSimdRealf MinImpulseNormal = SimdMin(SimdNegate(SimdDivide(ManifoldPoint.SimdNetPushOutNormal, Dt)), FSimdRealf::Zero());
					const FSimdSelector ShouldClampImpulse = SimdLess(SimdAdd(ManifoldPoint.SimdNetImpulseNormal, ImpulseNormal), MinImpulseNormal);
					const FSimdRealf ClampedImpulseNormal = SimdSubtract(MinImpulseNormal, ManifoldPoint.SimdNetImpulseNormal);
					ImpulseNormal = SimdSelect(ShouldClampImpulse, ClampedImpulseNormal, ImpulseNormal);

					// Calculate the impulse multipler if clamped to the dynamic friction cone
					const FSimdRealf MaxNetImpulseAndPushOutTangent = SimdAdd(ManifoldPoint.SimdNetImpulseNormal, SimdAdd(ImpulseNormal, SimdDivide(ManifoldPoint.SimdNetPushOutNormal, Dt)));
					const FSimdRealf MaxImpulseTangent = SimdMax(FSimdRealf::Zero(), SimdMultiply(SimdVelocityFriction, MaxNetImpulseAndPushOutTangent));
					const FSimdRealf MaxImpulseTangentSq = SimdSquare(MaxImpulseTangent);
					const FSimdRealf ImpulseTangentSq = SimdAdd(SimdSquare(ImpulseTangentU), SimdSquare(ImpulseTangentV));
					const FSimdRealf ImpulseTangentScale = SimdMultiply(MaxImpulseTangent, SimdInvSqrt(ImpulseTangentSq));

					// Apply the multiplier to lanes that exceeded the friction limit
					const FSimdSelector ExceededFrictionCone = SimdGreater(ImpulseTangentSq, SimdAdd(MaxImpulseTangentSq, FSimdRealf::Make(UE_SMALL_NUMBER)));
					ImpulseTangentU = SimdSelect(ExceededFrictionCone, SimdMultiply(ImpulseTangentScale, ImpulseTangentU), ImpulseTangentU);
					ImpulseTangentV = SimdSelect(ExceededFrictionCone, SimdMultiply(ImpulseTangentScale, ImpulseTangentV), ImpulseTangentV);

					// Clear the impulse for lanes that should not be solving for velocity
					ImpulseNormal = SimdSelect(ShouldSolveVelocity, ImpulseNormal, FSimdRealf::Zero());
					ImpulseTangentU = SimdSelect(ShouldSolveVelocity, ImpulseTangentU, FSimdRealf::Zero());
					ImpulseTangentV = SimdSelect(ShouldSolveVelocity, ImpulseTangentV, FSimdRealf::Zero());

					ManifoldPoint.SimdNetImpulseNormal = SimdAdd(ManifoldPoint.SimdNetImpulseNormal, ImpulseNormal);
					ManifoldPoint.SimdNetImpulseTangentU = SimdAdd(ManifoldPoint.SimdNetImpulseTangentU, ImpulseTangentU);
					ManifoldPoint.SimdNetImpulseTangentV = SimdAdd(ManifoldPoint.SimdNetImpulseTangentU, ImpulseTangentV);

					// NOTE: order of operations matches FPBDCollisionSolver::ApplyVelocityCorrection for AB Testing
					const FSimdVec3f Impulse = SimdAdd(SimdMultiply(ImpulseNormal, ManifoldPoint.SimdContactNormal), SimdAdd(SimdMultiply(ImpulseTangentU, ManifoldPoint.SimdContactTangentU), SimdMultiply(ImpulseTangentV, ManifoldPoint.SimdContactTangentV)));
					const FSimdVec3f DV0 = SimdMultiply(Impulse, SimdInvM0);
					const FSimdVec3f DV1 = SimdMultiply(Impulse, SimdInvM1);
					const FSimdVec3f DW0 = SimdAdd(SimdMultiply(ImpulseNormal, ManifoldPoint.SimdContactNormalAngular0), SimdAdd(SimdMultiply(ImpulseTangentU, ManifoldPoint.SimdContactTangentUAngular0), SimdMultiply(ImpulseTangentV, ManifoldPoint.SimdContactTangentVAngular0)));
					const FSimdVec3f DW1 = SimdAdd(SimdMultiply(ImpulseNormal, ManifoldPoint.SimdContactNormalAngular1), SimdAdd(SimdMultiply(ImpulseTangentU, ManifoldPoint.SimdContactTangentUAngular1), SimdMultiply(ImpulseTangentV, ManifoldPoint.SimdContactTangentVAngular1)));
					V0 = SimdAdd(V0, DV0);
					W0 = SimdAdd(W0, DW0);
					V1 = SimdSubtract(V1, DV1);
					W1 = SimdSubtract(W1, DW1);
				}

				ScatterBodyVelocities(V0, W0, V1, W1, Body0, Body1);
			}

			void SolveVelocityWithFriction(
				const TArrayView<FSimdManifoldPoint>& ManifoldPointsBuffer,
				const FSimdSolverBodyPtr& Body0,
				const FSimdSolverBodyPtr& Body1,
				const FSimdRealf& Dt,
				const FSimd4Realf& FrictionStiffnessScale)
			{
				// If all the lanes have zero velocity friction, run the zero-friction path
				// (only spheres and capsules use the velocity-based dynamic friction path)
				const FSimdSelector HasNonZeroFriction = SimdGreater(SimdVelocityFriction, FSimdRealf::Zero());
				if (!SimdAnyTrue(HasNonZeroFriction))
				{
					SolveVelocityNoFriction(ManifoldPointsBuffer, Body0, Body1, Dt);
					return;
				}

				SolveVelocityWithFrictionImpl(ManifoldPointsBuffer, Body0, Body1, Dt, FrictionStiffnessScale);
			}

			FSolverVec3 GetNetPushOut(
				const int32 ManifoldPointIndex,
				const int32 LaneIndex,
				const TArrayView<FSimdManifoldPoint>& ManifoldPointsBuffer) const
			{
				const int32 BufferIndex = GetBufferIndex(ManifoldPointIndex);
				const FSimdManifoldPoint& ManifoldPoint = ManifoldPointsBuffer[BufferIndex];

				return ManifoldPoint.SimdNetPushOutNormal.GetValue(LaneIndex) * ManifoldPoint.SimdContactNormal.GetValue(LaneIndex) +
					ManifoldPoint.SimdNetPushOutTangentU.GetValue(LaneIndex) * ManifoldPoint.SimdContactTangentU.GetValue(LaneIndex) +
					ManifoldPoint.SimdNetPushOutTangentV.GetValue(LaneIndex) * ManifoldPoint.SimdContactTangentV.GetValue(LaneIndex);
			}

			FSolverVec3 GetNetImpulse(
				const int32 ManifoldPointIndex,
				const int32 LaneIndex,
				const TArrayView<FSimdManifoldPoint>& ManifoldPointsBuffer) const
			{
				const int32 BufferIndex = GetBufferIndex(ManifoldPointIndex);
				const FSimdManifoldPoint& ManifoldPoint = ManifoldPointsBuffer[BufferIndex];

				return ManifoldPoint.SimdNetImpulseNormal.GetValue(LaneIndex) * ManifoldPoint.SimdContactNormal.GetValue(LaneIndex) +
					ManifoldPoint.SimdNetImpulseTangentU.GetValue(LaneIndex) * ManifoldPoint.SimdContactTangentU.GetValue(LaneIndex) +
					ManifoldPoint.SimdNetImpulseTangentV.GetValue(LaneIndex) * ManifoldPoint.SimdContactTangentV.GetValue(LaneIndex);
			}

			FSolverReal GetStaticFrictionRatio(
				const int32 ManifoldPointIndex,
				const int32 LaneIndex,
				const TArrayView<FSimdManifoldPoint>& ManifoldPointsBuffer) const
			{
				const int32 BufferIndex = GetBufferIndex(ManifoldPointIndex);
				const FSimdManifoldPoint& ManifoldPoint = ManifoldPointsBuffer[BufferIndex];

				return ManifoldPoint.SimdStaticFrictionRatio.GetValue(LaneIndex);
			}

		public:
			int32 GetBufferIndex(const int32 ManifoldPointIndex) const
			{
				return ManifoldPointBeginIndex + ManifoldPointIndex;
			}

			void Init()
			{
				MaxManifoldPoints = 0;
				ManifoldPointBeginIndex = INDEX_NONE;
				SimdNumManifoldPoints.SetValues(0);
				SimdStaticFriction.SetValues(0);
				SimdDynamicFriction.SetValues(0);
				SimdVelocityFriction.SetValues(0);
				SimdStiffness.SetValues(1);
				SimdInvM0.SetValues(0);
				SimdInvM1.SetValues(0);
			}

			// Each lane has space for MaxManifoldPoints, but not all will be used
			// Unused manifold points may be in the middle of the list (if disabled for example)
			int32 MaxManifoldPoints;
			int32 ManifoldPointBeginIndex;
			FSimdInt32 SimdNumManifoldPoints;

			FSimdRealf SimdStaticFriction;
			FSimdRealf SimdDynamicFriction;
			FSimdRealf SimdVelocityFriction;
			FSimdRealf SimdStiffness;
			FSimdRealf SimdInvM0;
			FSimdRealf SimdInvM1;
		};

		//////////////////////////////////////////////////////////////////////////////////////////////////
		//////////////////////////////////////////////////////////////////////////////////////////////////
		//////////////////////////////////////////////////////////////////////////////////////////////////
		//////////////////////////////////////////////////////////////////////////////////////////////////


		/**
		 * A helper for solving arrays of constraints.
		 * @note Only works with 4 SIMD lanes for now.
		 */
		class FPBDCollisionSolverHelperSimd
		{
		public:
			template<int TNumLanes>
			static void SolvePositionNoFriction(
				const TArrayView<TPBDCollisionSolverSimd<TNumLanes>>& Solvers,
				const TArrayView<TPBDCollisionSolverManifoldPointsSimd<TNumLanes>>& ManifoldPoints,
				const TArrayView<TSolverBodyPtrPairSimd<TNumLanes>>& SolverBodies,
				const FSolverReal Dt,
				const FSolverReal MaxPushOut);

			template<int TNumLanes>
			static void SolvePositionWithFriction(
				const TArrayView<TPBDCollisionSolverSimd<TNumLanes>>& Solvers,
				const TArrayView<TPBDCollisionSolverManifoldPointsSimd<TNumLanes>>& ManifoldPoints,
				const TArrayView<TSolverBodyPtrPairSimd<TNumLanes>>& SolverBodies,
				const FSolverReal Dt,
				const FSolverReal MaxPushOut);

			template<int TNumLanes>
			static void SolveVelocityNoFriction(
				const TArrayView<TPBDCollisionSolverSimd<TNumLanes>>& Solvers,
				const TArrayView<TPBDCollisionSolverManifoldPointsSimd<TNumLanes>>& ManifoldPoints,
				const TArrayView<TSolverBodyPtrPairSimd<TNumLanes>>& SolverBodies,
				const FSolverReal Dt);

			template<int TNumLanes>
			static void SolveVelocityWithFriction(
				const TArrayView<TPBDCollisionSolverSimd<TNumLanes>>& Solvers,
				const TArrayView<TPBDCollisionSolverManifoldPointsSimd<TNumLanes>>& ManifoldPoints,
				const TArrayView<TSolverBodyPtrPairSimd<TNumLanes>>& SolverBodies,
				const FSolverReal Dt);

			static CHAOS_API void CheckISPC();
		};

		template<> 
		void FPBDCollisionSolverHelperSimd::SolvePositionNoFriction(
			const TArrayView<TPBDCollisionSolverSimd<4>>& Solvers,
			const TArrayView<TPBDCollisionSolverManifoldPointsSimd<4>>& ManifoldPoints,
			const TArrayView<TSolverBodyPtrPairSimd<4>>& SolverBodies,
			const FSolverReal Dt,
			const FSolverReal MaxPushOut);

		template<>
		void FPBDCollisionSolverHelperSimd::SolvePositionWithFriction(
			const TArrayView<TPBDCollisionSolverSimd<4>>& Solvers,
			const TArrayView<TPBDCollisionSolverManifoldPointsSimd<4>>& ManifoldPoints,
			const TArrayView<TSolverBodyPtrPairSimd<4>>& SolverBodies,
			const FSolverReal Dt,
			const FSolverReal MaxPushOut);

		template<>
		void FPBDCollisionSolverHelperSimd::SolveVelocityNoFriction(
			const TArrayView<TPBDCollisionSolverSimd<4>>& Solvers,
			const TArrayView<TPBDCollisionSolverManifoldPointsSimd<4>>& ManifoldPoints,
			const TArrayView<TSolverBodyPtrPairSimd<4>>& SolverBodies,
			const FSolverReal Dt);

		template<>
		void FPBDCollisionSolverHelperSimd::SolveVelocityWithFriction(
			const TArrayView<TPBDCollisionSolverSimd<4>>& Solvers,
			const TArrayView<TPBDCollisionSolverManifoldPointsSimd<4>>& ManifoldPoints,
			const TArrayView<TSolverBodyPtrPairSimd<4>>& SolverBodies,
			const FSolverReal Dt);

	}	// namespace Private
}	// namespace Chaos

