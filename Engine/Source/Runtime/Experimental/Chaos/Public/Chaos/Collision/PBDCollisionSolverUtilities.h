// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/Defines.h"
#include "Chaos/Evolution/SolverBody.h"


namespace Chaos
{
	namespace Private
	{

		inline bool CalculateBodyShockPropagation(
			const FSolverBody& Body0,
			const FSolverBody& Body1,
			const FSolverReal ShockPropagation,
			FSolverReal& OutShockPropagation0,
			FSolverReal& OutShockPropagation1)
		{
			OutShockPropagation0 = FSolverReal(1);
			OutShockPropagation1 = FSolverReal(1);

			// Shock propagation decreases the inverse mass of bodies that are lower in the pile
			// of objects. This significantly improves stability of heaps and stacks. Height in the pile is indictaed by the "level". 
			// No need to set an inverse mass scale if the other body is kinematic (with inv mass of 0).
			// Bodies at the same level do not take part in shock propagation.
			if (Body0.IsDynamic() && Body1.IsDynamic() && (Body0.Level() != Body1.Level()))
			{
				// Set the inv mass scale of the "lower" body to make it heavier
				if (Body0.Level() < Body1.Level())
				{
					OutShockPropagation0 = ShockPropagation;
				}
				else
				{
					OutShockPropagation1 = ShockPropagation;
				}
				return true;
			}

			return false;
		}

	} // namespace Private
} // namespace Chaos