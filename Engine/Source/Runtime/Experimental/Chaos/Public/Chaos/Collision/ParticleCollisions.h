// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Chaos/Core.h"

#include "Chaos/Collision/CollisionVisitor.h"
#include "Chaos/ParticleHandleFwd.h"

namespace Chaos
{
	class FParticlePairMidPhase;

	/**
	 * @brief Knows about all the collisions detectors associated with a particular particle.
	 * Used when particles are destroyed to remove perisstent collisions from the system, or
	 * when Islands are woken to restore the collisions.
	 * 
	*/
	class FParticleCollisions
	{
	public:
		// In a mostly stationary scene the choice of container doesn't matter much. In a highly dynamic
		// scene it matters a lot. This was tested with TMap, TSortedMap, and TArray. TArray was better in all cases.
		// We store the list of overlapping particle pairs for every particle. This list can get quite
		// large for world objects. We need to make sure that Add / Remove are not O(N).
		// We also search this list to find an existing MidPhase for a specific particle pair. However
		// we always search the particle with the fewest overlaps, which will be a dynamic particle
		// that should not have too many contacts. 
		using FContainerType = TArray<TPair<uint64, FParticlePairMidPhase*>>;

		CHAOS_API FParticleCollisions();
		CHAOS_API ~FParticleCollisions();

		inline int32 Num() const 
		{ 
			return MidPhases.Num();
		}

		/**
		 * @brief Clear the list of midphases. Only for use in shutdown.
		*/
		inline void Reset()
		{
			MidPhases.Reset();
		}

		/**
		 * @brief Add a mid phase to the list
		 * We are passing the particle and midphase here rather than just the key because we store a cookie
		 * on the midphase that we want to retrieve, and it has one cookie per particle. 
		 * This could probably be cleaned up a bit...
		*/
		CHAOS_API void AddMidPhase(FGeometryParticleHandle* InParticle, FParticlePairMidPhase* InMidPhase);

		/**
		 * @brief Remove a mid phase
		*/
		CHAOS_API void RemoveMidPhase(FGeometryParticleHandle* InParticle, FParticlePairMidPhase* InMidPhase);

		/**
		 * @brief Get a midphase by its index
		 */
		inline FParticlePairMidPhase* GetMidPhase(const int32 InIndex)
		{
			if (MidPhases.IsValidIndex(InIndex))
			{
				return MidPhases[InIndex].Value;
			}
			return nullptr;
		}

		/**
		 * @brief Find the mid phase with the matching key
		 * @param InKey The internal key from a FCollisionParticlePairKey
		 * @todo(chaos): we should use FCollisionParticlePairKey here
		*/
		inline FParticlePairMidPhase* FindMidPhase(const uint64 InKey)
		{
			for (int32 Index = 0; Index < MidPhases.Num(); ++Index)
			{
				if (MidPhases[Index].Key == InKey)
				{
					return MidPhases[Index].Value;
				}
			}
			return nullptr;
		}

		/**
		 * @brief Visit all of the midphases on the particle and call the specified function
		 * @tparam TLambda visitor type with signature ECollisionVisitorResult(FParticlePairMidPhase&)
		 *
		 * @note Do not call RemoveMidPhase from the visitor
		*/
		template<typename TLambda>
		inline ECollisionVisitorResult VisitMidPhases(const TLambda& Lambda);

		/**
		 * @brief Visit all of the midphases on the particle and call the specified function
		 * @tparam TLambda visitor type with signature void(const FParticlePairMidPhase&)
		 * 
		 * @note Do not call RemoveMidPhase from the visitor
		*/
		template<typename TLambda>
		inline ECollisionVisitorResult VisitConstMidPhases(const TLambda& Lambda) const;

		/**
		 * @brief Visit all the collisions on this particle
		 * @tparam TLambda visitor type with signature void(FPBDCollisionConstraint&)
		 * 
		 * @note do not delete constraint from the lambda. You may disable them though.
		*/
		template<typename TLambda>
		inline ECollisionVisitorResult VisitCollisions(const TLambda& Visitor, const ECollisionVisitorFlags VisitFlags = ECollisionVisitorFlags::VisitDefault);

		/**
		 * @brief Visit all the collisions on this particle
		 * @tparam TLambda visitor type with signature void(const FPBDCollisionConstraint&)
		 * 
		 * @note do not delete constraint from the lambda. You may disable them though.
		*/
		template<typename TLambda>
		inline ECollisionVisitorResult VisitConstCollisions(const TLambda& Visitor, const ECollisionVisitorFlags VisitFlags = ECollisionVisitorFlags::VisitDefault) const;

	private:
		FContainerType MidPhases;
	};

}
