// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"

#include "Chaos/CollisionResolutionTypes.h"
#include "Chaos/Collision/CollisionKeys.h"
#include "Chaos/Collision/CollisionVisitor.h"
#include "Chaos/Collision/PBDCollisionConstraint.h"
#include "Chaos/ImplicitFwd.h"
#include "Chaos/ParticleHandleFwd.h"
#include "ProfilingDebugging/CsvProfiler.h"


class FChaosVDDataWrapperUtils;

namespace Chaos
{
	namespace Private
	{
		class FCollisionConstraintAllocator;
	}

	class FParticlePairMidPhaseCollisionKey;

	/**
	 * The type of the particle pair midphase.
	 */
	enum EParticlePairMidPhaseType : int8
	{
		// A general purpose midphase that handle BVHs, Meshes, 
		// Unions of Unions, etc in the geometry hierarchy.
		Generic,

		// A midphase optimized for particle pairs with a small
		// number of shapes. Pre-expands the set of potentially
		// colliding shape pairs.
		ShapePair,

		// A midphase used to collide particles as sphere approximations
		SphereApproximation,
	};

	/**
	 * @brief Handles collision detection for a pair of simple shapes (i.e., not compound shapes)
	 * 
	 * This is used by FShapePairParticlePairMidPhase
	 */
	class FSingleShapePairCollisionDetector
	{
	public:
		using FCollisionsArray = TArray<FPBDCollisionConstraint*, TInlineAllocator<1>>;

		CHAOS_API FSingleShapePairCollisionDetector(
			FGeometryParticleHandle* InParticle0,
			const FPerShapeData* InShape0,
			FGeometryParticleHandle* InParticle1,
			const FPerShapeData* InShape1,
			const Private::FCollisionSortKey& InCollisionSortKey,
			const EContactShapesType InShapePairType, 
			FParticlePairMidPhase& MidPhase);
		CHAOS_API FSingleShapePairCollisionDetector(FSingleShapePairCollisionDetector&& R);
		FSingleShapePairCollisionDetector(const FSingleShapePairCollisionDetector& R) = delete;
		FSingleShapePairCollisionDetector& operator=(const FSingleShapePairCollisionDetector& R) = delete;
		CHAOS_API ~FSingleShapePairCollisionDetector();

		const FPBDCollisionConstraint* GetConstraint() const{ return Constraint.Get(); }
		FPBDCollisionConstraint* GetConstraint() { return Constraint.Get(); }
		const FGeometryParticleHandle* GetParticle0() const { return Particle0; }
		FGeometryParticleHandle* GetParticle0() { return Particle0; }
		const FGeometryParticleHandle* GetParticle1() const { return Particle1; }
		FGeometryParticleHandle* GetParticle1() { return Particle1; }
		const FPerShapeData* GetShape0() const { return Shape0; }
		const FPerShapeData* GetShape1() const { return Shape1; }

		/**
		 * @brief Have we run collision detection since this Epoch (inclusive)
		*/
		inline bool IsUsedSince(const int32 Epoch) const
		{
			return Constraint.IsValid() && (LastUsedEpoch >= Epoch);
		}

		/**
		 * @brief Perform a bounds check and run the narrow phase if necessary
		 * @return The number of collisions constraints that were activated
		*/
		UE_DEPRECATED(5.4, "Use single precision version")
		int32 GenerateCollision(const FReal CullDistance, const FReal Dt, const FCollisionContext& Context) { return GenerateCollision(FRealSingle(Dt), FRealSingle(CullDistance), FVec3f(0), Context); }

		CHAOS_API int32 GenerateCollision(
			const FRealSingle Dt,
			const FRealSingle CullDistance,
			const FVec3f& RelativeMovement,
			const FCollisionContext& Context);

		/**
		 * @brief Generate a SweptConstraint as long as AABBs overlap
		 * @return The number of collisions constraints that were activated
		*/
		UE_DEPRECATED(5.4, "Use single precision version")
		int32 GenerateCollisionCCD( const bool bEnableCCDSweep, const FReal CullDistance, const FReal Dt, const FCollisionContext& Context) { return GenerateCollisionCCD(FRealSingle(Dt), FRealSingle(CullDistance), FVec3f(0), bEnableCCDSweep, Context); }

		CHAOS_API int32 GenerateCollisionCCD(
			const FRealSingle Dt,
			const FRealSingle CullDistance,
			const FVec3f& RelativeMovement,
			const bool bEnableCCDSweep,
			const FCollisionContext& Context);

		/**
		 * @brief Reactivate the constraint
		 * @parame SleepEpoch The tick on which the particle went to sleep.
		 * Only constraints that were active when the particle went to sleep should be reactivated.
		*/
		CHAOS_API void WakeCollision(
			const int32 SleepEpoch, 
			const int32 CurrentEpoch);

		/**
		 * @brief Set the collision from the parameter and activate it
		 * This is used by the Resim restore functionality
		*/
		CHAOS_API void SetCollision(
			const FPBDCollisionConstraint& Constraint, 
			const FCollisionContext& Context);

	private:
		CHAOS_API int32 GenerateCollisionImpl(
			const FRealSingle Dt,
			const FRealSingle CullDistance,
			const FVec3f& RelativeMovement,
			const FCollisionContext& Context);
			
		CHAOS_API int32 GenerateCollisionCCDImpl(
			const FRealSingle Dt,
			const FRealSingle CullDistance,
			const FVec3f& RelativeMovement,
			const bool bEnableCCDSweep,
			const FCollisionContext& Context);

		CHAOS_API int32 GenerateCollisionProbeImpl(
			const FRealSingle Dt,
			const FRealSingle CullDistance,
			const FVec3f& RelativeMovement,
			const FCollisionContext& Context);

		/**
		 * @brief Whether the two shapes are separated by less than CullDistance (i.e., we should run the narrow phase).
		 * Also returns true if bounds checking is disabled (globally or for this pair)
		*/
		CHAOS_API bool DoBoundsOverlap(
			const FRealSingle CullDistance, 
			const FVec3f& RelativeMovement,
			const int32 CurrentEpoch);

		/**
		 * @brief Create a constraint
		*/
		CHAOS_API void CreateConstraint(
			const FReal CullDistance, 
			const FCollisionContext& Context);

		FParticlePairMidPhase& MidPhase;
		FPBDCollisionConstraintPtr Constraint;
		FGeometryParticleHandle* Particle0;
		FGeometryParticleHandle* Particle1;
		const FPerShapeData* Shape0;
		const FPerShapeData* Shape1;
		Private::FCollisionSortKey CollisionSortKey;
		FRealSingle SphereBoundsCheckSize;
		int32 LastUsedEpoch;
		EContactShapesType ShapePairType;
		Private::FImplicitBoundsTestFlags BoundsTestFlags;
	};


	/**
	 * @brief Produce collisions for a particle pair
	 * A FParticlePairMidPhase object is created for every particle pair whose bounds overlap. It is 
	 * responsible for building a set of potentially colliding shape pairs and running collision
	 * detection on those pairs each tick.
	 * 
	 * @note The lifetime of midphase objects is handled entirely by the CollisionConstraintAllocator. 
	 * Nothing outside of the CollisionConstraintAllocator should hold a pointer to the detector 
	 * or any constraints it creates for more than the duration of the tick.
	 * 
	 * @see FShapePairParticlePairMidPhase, FGenericParticlePairMidPhase
	 * 
	 */
	class FParticlePairMidPhase
	{
	public:
		static CHAOS_API EParticlePairMidPhaseType CalculateMidPhaseType(FGeometryParticleHandle* InParticle0, FGeometryParticleHandle* InParticle1);
		static CHAOS_API FParticlePairMidPhase* Make(FGeometryParticleHandle* InParticle0, FGeometryParticleHandle* InParticle1);

		UE_NONCOPYABLE(FParticlePairMidPhase);

		CHAOS_API FParticlePairMidPhase(const EParticlePairMidPhaseType InMidPhaseType);
		CHAOS_API virtual ~FParticlePairMidPhase();

		EParticlePairMidPhaseType GetMidPhaseType() const
		{
			return MidPhaseType;
		}

		/**
		 * @brief Set up the midphase based on the ShapesArrays of the two particles
		 * Only intended to be called once right after constructor. We don't do this work in
		 * the constructor so that we can reduce the time that the lock is held when allocating
		 * new MidPhases.
		*/
		CHAOS_API void Init(
			FGeometryParticleHandle* InParticle0,
			FGeometryParticleHandle* InParticle1,
			const Private::FCollisionParticlePairKey& InKey,
			const FCollisionContext& Context);

		inline FGeometryParticleHandle* GetParticle0() const { return Particle0; }

		inline FGeometryParticleHandle* GetParticle1() const { return Particle1; }

		inline const Private::FCollisionParticlePairKey& GetKey() const { return ParticlePairKey; }

		inline bool IsValid() const { return (Particle0 != nullptr) && (Particle1 != nullptr); }

		// Prefetch the memory in this class for later use
		inline void CachePrefetch()
		{
			FPlatformMisc::PrefetchBlock(this, sizeof(*this));
		}

		/**
		 * @brief Have we run collision detection since this Epoch (inclusive)
		*/
		inline bool IsUsedSince(const int32 Epoch) const
		{
			return (LastUsedEpoch >= Epoch);
		}


		/**
		 * @brief Whether the particle pair is sleeping and therefore contacts should not be culled (they will be reused on wake)
		*/
		inline bool IsSleeping() const { return Flags.bIsSleeping; }

		/**
		 * @brief Update the sleeping state
		 * If this switches the state to Awake, it will reactivate any collisions between the particle pair that
		 * were active when they went to sleep.
		*/
		CHAOS_API void SetIsSleeping(const bool bInIsSleeping, const int32 CurrentEpoch);

		CHAOS_API bool IsInConstraintGraph() const;

		/**
		 * @brief Destroy all collisions and prevent this midphasae from being used any more. Called when one of its particles is destoyed.
		 * It will be culled at the next Prune in the CollisionConstraintAllocator. We don't delete it immediately so that we don't
		 * have to remove it from either Particle's ParticleCollisions array (which is O(N) and unnecessary when the particles are being destroyed)
		*/
		CHAOS_API void DetachParticle(FGeometryParticleHandle* Particle);

		/**
		 * @brief Delete all cached data and collisions. Should be called when a particle changes its shapes
		*/
		CHAOS_API void Reset();

		/**
		 *
		 */
		CHAOS_API void ResetModifications();

		/**
		 * @brief Create collision constraints for all colliding shape pairs
		*/
		CHAOS_API void GenerateCollisions(
			const FReal CullDistance,
			const FReal Dt,
			const FCollisionContext& Context);

		/**
		 * @brief Copy a collision and activate it
		 * This is used by the Resim system to restore saved colisions. If there is already a matching constraint
		 * it will be overwritten, otherwise a new constraint will be added.
		*/
		CHAOS_API void InjectCollision(const FPBDCollisionConstraint& Constraint, const FCollisionContext& Context);

		/**
		 * @brief Call a lambda on each active collision constraint
		 * This includes sleeping constraints, but skips constraints that are were not used on the last awake tick
		 * but are still kept around as an optimization.
		 * @param Visitor functor with signature ECollisionVisitorResult(FPBDCollisionConstraint& Constraint)
		*/
		template<typename TLambda>
		ECollisionVisitorResult VisitCollisions(const TLambda& Visitor, const ECollisionVisitorFlags VisitFlags = ECollisionVisitorFlags::VisitDefault);


		/**
		 * @brief Call a lambda on each active collision constraint
		 * @param Visitor functor with signature ECollisionVisitorResult(const FPBDCollisionConstraint& Constraint)
		 */
		template<typename TLambda>
		ECollisionVisitorResult VisitConstCollisions(const TLambda& Visitor, const ECollisionVisitorFlags VisitFlags = ECollisionVisitorFlags::VisitDefault) const;

		/**
		 * @brief Cookie for use by FParticleCollisions
		*/
		int32 GetParticleCollisionsIndex(FGeometryParticleHandle* InParticle) const
		{
			check((InParticle == Particle0) || (InParticle == Particle1));
			if (InParticle == Particle0)
			{
				return ParticleCollisionsIndex0;
			}
			else
			{
				return ParticleCollisionsIndex1;
			}
		}

		/**
		 * @brief Cookie for use by FParticleCollisions
		*/
		void SetParticleCollisionsIndex(FGeometryParticleHandle* InParticle, const int32 InIndex)
		{
			check((InParticle == Particle0) || (InParticle == Particle1));
			if (InParticle == Particle0)
			{
				ParticleCollisionsIndex0 = InIndex;
			}
			else
			{
				ParticleCollisionsIndex1 = InIndex;
			}
		}

		/**
		 * @brief By default midphases are active. If IsActive is set to false,
		 * this midphase will not generate a narrow phase.
		 */
		void SetIsActive(bool bIsActive)
		{
			Flags.bIsModified = true;
			Flags.bIsActive = bIsActive;
		}

		/**
		 * @brief Override the CCD condition for this mid-phase. Used by the MidPhase modifier and gets reset every frame.
		 */
		void SetCCDIsActive(bool bCCDIsActive)
		{
			Flags.bIsModified = true;
			Flags.bIsCCDActive = bCCDIsActive;
		}

		UE_DEPRECATED(5.6, "Use SetConvexOptimizationIsActive which takes an OnDestroyingCollision functor")
		void SetConvexOptimizationIsActive(bool bSetConvexOptimizationIsActive)
		{
			SetConvexOptimizationIsActive(bSetConvexOptimizationIsActive, [](FPBDCollisionConstraint*) {});
		}


		/**
		 * @brief Override whether convex optimization if used this mid-phase. Used by the MidPhase modifier and gets reset every frame.
		 * NOTE: when this setting is changed, collisions will be destroyed so any external systems referencing them will need to 
		 * drop their references.
		 */
		virtual void SetConvexOptimizationIsActive(bool bSetConvexOptimizationIsActive, TFunctionRef<void(FPBDCollisionConstraint*)> OnDestroyingCollision) {}

		/*
		 * True if CCD is supported by either particle
		 */
		bool IsCCD() const
		{
			return Flags.bIsCCD;
		}

		/*
		 * True if CCD is active for this midphase on this frame.This can be changed by modifiers and resets to bIsCCD each frame.
		 */
		bool IsCCDActive() const
		{
			return Flags.bIsCCDActive;
		}

	protected:

		virtual void ResetImpl() = 0;

		/**
		 * @brief Build the list of potentially colliding shape pairs.
		 * This is all the shape pairs in the partilces' shapes arrays that pass the collision filter.
		*/
		virtual void BuildDetectorsImpl() = 0;

		virtual int32 GenerateCollisionsImpl(
			const FRealSingle Dt,
			const FRealSingle CullDistance,
			const FVec3f& RelativeMovement,
			const FCollisionContext& Context) = 0;

		virtual void WakeCollisionsImpl(const int32 CurrentEpoch) = 0;

		virtual void InjectCollisionImpl(const FPBDCollisionConstraint& Constraint, const FCollisionContext& Context) = 0;

		/**
		 * @brief Decide whether we should have CCD enabled on this constraint
		 * @return true if CCD is enabled this tick, false otherwise
		 * This may return false, even for collisions on CCD-enabled bodies when the bodies are moving slowly
		 */
		CHAOS_API bool ShouldEnableCCDSweep(const FReal Dt);

		CHAOS_API void InitThresholds();


		union FFlags
		{
			FFlags() : Bits(0) {}
			struct
			{
				uint16 bIsActive : 1;    // True if this midphase should generate a narrow phase at all
				uint16 bIsCCD : 1;       // True if CCD is supported by either particle
				uint16 bIsCCDActive : 1; // True if CCD is active for this midphase on this frame. This can be changed by modifiers and resets to bIsCCD each frame.
				uint16 bUseSweep : 1;    // True if CCD is active (this frame) and we are moving fast enough to require a sweep
				uint16 bIsMACD : 1;      // True if MACD (movement-aware collision detection) is enabled for this pair
				uint16 bIsConvexOptimizationActive : 1; // True if convex optimization is active for this midphase
				uint16 bIsConvexOptimizationUsed : 1;   // True if we generated collisions with convex optimization active (used to detect changes to bIsConvexOptimizationActive in modifiers). See FGenericParticlePairMidPhase::GenerateCollisionsImpl
				uint16 bIsSleeping : 1;  // True is the particles in the midphase are sleeping
				uint16 bIsModified : 1;  // True if a modifier applied any changes to this midphase
			};
			uint16 Bits;
		};

		// VTable Ptr									// 8 bytes
		EParticlePairMidPhaseType MidPhaseType;			// 1 byte
		FFlags Flags;									// 2 bytes

		FGeometryParticleHandle* Particle0;				// 8 bytes
		FGeometryParticleHandle* Particle1;				// 8 bytes
		// A number based on the size of the dynamic objects used to scale cull distance
		FRealSingle CullDistanceScale;					// 4 bytes

		Private::FCollisionParticlePairKey ParticlePairKey;		// 8 bytes

		int32 LastUsedEpoch;							// 4 bytes
		int32 NumActiveConstraints;						// 4 bytes

		// Indices into the arrays of collisions on the particles. This is a cookie for use by FParticleCollisions
		int32 ParticleCollisionsIndex0;					// 4 bytes
		int32 ParticleCollisionsIndex1;					// 4 bytes
		
		friend ::FChaosVDDataWrapperUtils;
	};


	/**
	 * A midphase for a particle pair that pre-builds a set of all potentially colliding shape
	 * pairs. This is the fast path used when each particle has a small number of shapes and
	 * does not contain a complex hierarchy. This midphase (compared to FGenericParticlePairMidPhase)
	 * caches various data like the results of the collision filtering, the shapr pair types, etc.
	 */
	class FShapePairParticlePairMidPhase : public FParticlePairMidPhase
	{
	public:
		friend class FParticlePairMidPhase;

		CHAOS_API FShapePairParticlePairMidPhase();

		CHAOS_API virtual void ResetImpl() override final;
		CHAOS_API virtual void BuildDetectorsImpl() override final;

	protected:
		CHAOS_API virtual int32 GenerateCollisionsImpl(
			const FRealSingle Dt,
			const FRealSingle CullDistance,
			const FVec3f& RelativeMovement,
			const FCollisionContext& Context) override final;

		CHAOS_API virtual void WakeCollisionsImpl(const int32 CurrentEpoch) override final;

		CHAOS_API virtual void InjectCollisionImpl(const FPBDCollisionConstraint& Constraint, const FCollisionContext& Context) override final;

	private:
		CHAOS_API void TryAddShapePair(
			const FPerShapeData* Shape0, 
			const int32 ShapeIndex0, 
			const FPerShapeData* Shape1, 
			const int32 ShapeIndex1);

		TArray<FSingleShapePairCollisionDetector, TInlineAllocator<1>> ShapePairDetectors;	// 88 bytes
	};

	// A set of cached data about a simple implicit object on a particle
	struct FLeafImplicitObject;

	/**
	 * A midphase for a particle pair where one or both have a large number of collisions
	 * shapes, or a non-flat hierarchy of shapes. This is the general-purpose collision
	 * path and does not cache as much data as the FShapePairParticlePairMidPhase and
	 * must visit the geometry hierarchy on both shapes every time we detect collisions.
	 * It must also rerun the collision filters on overlapping pairs, among other things.
	 * It does, howver, take advantage of the BVH held in the root ImplicitObjectUnion
	 * if there is one, so it can be much faster the the FShapePairParticlePairMidPhase
	 * when the set of potentially colliding pairs is very large.
	 */
	class FGenericParticlePairMidPhase : public FParticlePairMidPhase
	{
	public:
		friend class FParticlePairMidPhase;

		CHAOS_API FGenericParticlePairMidPhase();
		CHAOS_API ~FGenericParticlePairMidPhase();

		CHAOS_API virtual void ResetImpl() override final;
		CHAOS_API virtual void BuildDetectorsImpl() override final;

	protected:
		CHAOS_API virtual int32 GenerateCollisionsImpl(
			const FRealSingle Dt,
			const FRealSingle CullDistance,
			const FVec3f& RelativeMovement,
			const FCollisionContext& Context) override final;

		CHAOS_API virtual void WakeCollisionsImpl(const int32 CurrentEpoch) override final;

		CHAOS_API virtual void InjectCollisionImpl(const FPBDCollisionConstraint& Constraint, const FCollisionContext& Context) override final;

	private:
		// BVH on ParticleA versus BVH on ParticleB
		CHAOS_API void GenerateCollisionsBVHBVH(
			FGeometryParticleHandle* ParticleA, const Private::FImplicitBVH* BVHA,
			FGeometryParticleHandle* ParticleB, const Private::FImplicitBVH* BVHB,
			const FReal CullDistance, const FReal Dt, const FCollisionContext& Context);

		// BVH on ParticleA versus implicit hierarchy of ParticleB
		CHAOS_API void GenerateCollisionsBVHImplicitHierarchy(
			FGeometryParticleHandle* ParticleA, const Private::FImplicitBVH* BVHA,
			FGeometryParticleHandle* ParticleB, const FImplicitObject* RootImplicitB, const Private::FConvexOptimizer* ConvexOptimizerB,
			const FReal CullDistance, const FReal Dt, const FCollisionContext& Context);

		// Implicit hierarchy of particle A versus implicit hierarchy of ParticleB (used when no BVH present on either)
		CHAOS_API void GenerateCollisionsImplicitHierarchyImplicitHierarchy(
			FGeometryParticleHandle* ParticleA, const FImplicitObject* RootImplicitA, const Private::FConvexOptimizer* ConvexOptimizerA,
			FGeometryParticleHandle* ParticleB, const FImplicitObject* RootImplicitB, const Private::FConvexOptimizer* ConvexOptimizerB,
			const FReal CullDistance, const FReal Dt, const FCollisionContext& Context);

		// BVH on particle A versus a Leaf Implicit of ParticleB
		CHAOS_API void GenerateCollisionsBVHImplicitLeaf(
			FGeometryParticleHandle* ParticleA, const Private::FImplicitBVH* BVHA,
			FGeometryParticleHandle* ParticleB, const FImplicitObject* ImplicitB, const FShapeInstance* ShapeInstanceB, const FRigidTransform3& RelativeTransformB, const int32 LeafObjectIndexB,
			const FReal CullDistance, const FReal Dt, const FCollisionContext& Context);

		// Leaf Implicit on ParticleA versus Leaf Implicit of ParticleB
		CHAOS_API void GenerateCollisionsImplicitLeafImplicitLeaf(
			FGeometryParticleHandle* ParticleA, const FImplicitObject* ImplicitA, const FShapeInstance* ShapeInstanceA, const FRigidTransform3 ParticleWorldTransformA, const FRigidTransform3& RelativeTransformA, const int32 LeafObjectIndexA,
			FGeometryParticleHandle* ParticleB, const FImplicitObject* ImplicitB, const FShapeInstance* ShapeInstanceB, const FRigidTransform3 ParticleWorldTransformB, const FRigidTransform3& RelativeTransformB, const int32 LeafObjectIndexB,
			const FReal CullDistance, const FReal Dt, const FCollisionContext& Context);

		// A bounds check between two implicits
		CHAOS_API bool DoBoundsOverlap(
			const FImplicitObject* ImplicitA, const FRigidTransform3& ParticleWorldTransformA, const FRigidTransform3& ShapeRelativeTransformA,
			const FImplicitObject* ImplicitB, const FRigidTransform3& ParticleWorldTransformB, const FRigidTransform3& ShapeRelativeTransformB,
			const Private::FImplicitBoundsTestFlags BoundsTestFlags, const FRealSingle DistanceCheckSize, const FReal CullDistance);

		CHAOS_API FPBDCollisionConstraint* FindOrCreateConstraint(
			FGeometryParticleHandle* InParticle0, const FImplicitObject* InImplicit0, const int32 InImplicitId0, const FShapeInstance* InShape0, const FBVHParticles* InBVHParticles0, const FRigidTransform3& InShapeRelativeTransform0,
			FGeometryParticleHandle* InParticle1, const FImplicitObject* InImplicit1, const int32 InImplicitId1, const FShapeInstance* InShape1, const FBVHParticles* InBVHParticles1, const FRigidTransform3& InShapeRelativeTransform1,
			const FReal CullDistance, const EContactShapesType ShapePairType, const bool bUseManifold, const bool bEnableSweep, const FCollisionContext& Context);

		CHAOS_API FPBDCollisionConstraint* FindConstraint(const FParticlePairMidPhaseCollisionKey& CollisionKey);

		CHAOS_API FPBDCollisionConstraint* CreateConstraint(
			FGeometryParticleHandle* Particle0, const FImplicitObject* Implicit0, const FPerShapeData* Shape0, const FBVHParticles* BVHParticles0, const FRigidTransform3& ShapeRelativeTransform0,
			FGeometryParticleHandle* Particle1, const FImplicitObject* Implicit1, const FPerShapeData* Shape1, const FBVHParticles* BVHParticles1, const FRigidTransform3& ShapeRelativeTransform1,
			const FParticlePairMidPhaseCollisionKey& CollisionKey, const Private::FCollisionSortKey& CollisionSortKey, 
			const FReal CullDistance, const EContactShapesType ShapePairType, const bool bInUseManifold, const FCollisionContext& Context);

		CHAOS_API int32 ProcessNewConstraints(const FReal CullDistance, const FReal Dt, const FCollisionContext& Context);

		CHAOS_API void PruneConstraints(const int32 CurrentEpoch);

		CHAOS_API bool UpdateCollision(FPBDCollisionConstraint* Constraint, const FReal CullDistance, const FReal Dt, const FCollisionContext& Context);

		CHAOS_API bool UpdateCollisionCCD( FPBDCollisionConstraint* Constraint, const FReal CullDistance, const FReal Dt, const FCollisionContext& Context);

		CHAOS_API virtual void SetConvexOptimizationIsActive(bool bSetConvexOptimizationIsActive, TFunctionRef<void(FPBDCollisionConstraint*)> OnDestroyingCollision) override final;

		TMap<uint64, FPBDCollisionConstraintPtr> Constraints;
		TArray<FPBDCollisionConstraint*> NewConstraints;
	};

	/**
	* A midphase for a particle pair that replaces both particles with a sphere approximation
	*/
	class FSphereApproximationParticlePairMidPhase : public FParticlePairMidPhase
	{
	public:
		friend class FParticlePairMidPhase;

		CHAOS_API FSphereApproximationParticlePairMidPhase();

		CHAOS_API virtual void ResetImpl() override final;
		CHAOS_API virtual void BuildDetectorsImpl() override final;

	protected:
		CHAOS_API virtual int32 GenerateCollisionsImpl(
			const FRealSingle Dt,
			const FRealSingle CullDistance,
			const FVec3f& RelativeMovement,
			const FCollisionContext& Context) override final;

		CHAOS_API virtual void WakeCollisionsImpl(const int32 CurrentEpoch) override final;

		CHAOS_API virtual void InjectCollisionImpl(const FPBDCollisionConstraint& Constraint, const FCollisionContext& Context) override final;

		static void InitSphere(const FGeometryParticleHandle* InParticle, FImplicitSphere3& OutSphere);

	private:
		FImplicitSphere3 Sphere0;
		FImplicitSphere3 Sphere1;
		const FShapeInstance* SphereShape0;
		const FShapeInstance* SphereShape1;
		FPBDCollisionConstraintPtr Constraint;
		int32 LastUsedEpoch;
		bool bHasSpheres;
	};
}
