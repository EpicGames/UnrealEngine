// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/Collision/ConvexContactPoint.h"
#include "Chaos/Collision/ConvexContactPointUtilities.h"
#include "Chaos/Convex.h"
#include "Chaos/Triangle.h"
#include "Chaos/Utilities.h"

namespace Chaos::Private
{
	// Find the contact point between a convex and a triangle using SAT.
	// NOTE: Does not fill in the features of OutContactPoint (see GetConvexFeature())
	template<typename ConvexType>
	bool SATConvexTriangle(const ConvexType& Convex, const FTriangle& Triangle, const FVec3& TriangleNormal, const FReal CullDistanceSq, Private::FConvexContactPoint& OutContactPoint)
	{
		// Constants. NOTE: We square and multiple InvalidPhi so not using lowest()
		const FReal NormalTolerance = FReal(1.e-8);
		const FReal NormalToleranceSq = NormalTolerance * NormalTolerance;
		const FReal InvalidPhi = FReal(-1.e10);	// 1000 km penetration

		// Triangle (same space as convex)
		const FVec3& TriN = TriangleNormal;
		const FVec3 TriC = Triangle.GetCentroid();

		//
		// SAT: Triangle face vs Convex verts
		//

		// We store the convex vertex distances to the triangle face for use in the edge-edge culling
		FMemMark Mark(FMemStack::Get());
		TArray<FReal, TMemStackAllocator<alignof(FReal)>> ConvexVertexDs;
		ConvexVertexDs.SetNum(Convex.NumVertices());
		TArrayView<FReal> ConvexVertexDsView = MakeArrayView(ConvexVertexDs);

		// Project the convex onto the triangle normal, with distances relative to the triangle plane
		FReal TriPlaneDMin, TriPlaneDMax;
		int32 ConvexVertexIndexMin, ConvexVertexIndexMax;
		Private::ProjectOntoAxis(Convex, TriN, TriC, TriPlaneDMin, TriPlaneDMax, ConvexVertexIndexMin, ConvexVertexIndexMax, &ConvexVertexDsView);

		// Distance culling
		if (Utilities::SignedSquare(TriPlaneDMin) > CullDistanceSq)
		{
			// Outside triangle face and separated by more than CullDistance
			return false;
		}
		if (TriPlaneDMax < FReal(0))
		{
			// Inside triangle face (single-sided collision)
			return false;
		}

		//
		// SAT: Convex faces vs triangle verts
		//

		// For each convex plane, project the Triangle onto the convex plane normal
		// and reject if the separation is more than cull distance.
		FVec3 ConvexPlaneN = FVec3(0);
		FVec3 ConvexPlaneX = FVec3(0);
		FReal ConvexPlaneDMin = InvalidPhi;
		int32 ConvexPlaneIndexMin = INDEX_NONE;
		int32 ConvexPlaneTriangleIndexMin = INDEX_NONE;
		for (int32 PlaneIndex = 0; PlaneIndex < Convex.NumPlanes(); ++PlaneIndex)
		{
			FVec3 ConN, ConX;
			Convex.GetPlaneNX(PlaneIndex, ConN, ConX);

			FReal DMin, DMax;
			int32 IndexMin, IndexMax;
			Private::ProjectOntoAxis(Triangle, ConN, ConX, DMin, DMax, IndexMin, IndexMax);

			// Backface cull - we can't select a convex face that would us a contact normal pointing below the triangle
			//const FReal ConvexDotTriangleNormal = FVec3::DotProduct(ConN, TriN);
			//if (ConvexDotTriangleNormal > 0)
			//{
			//	continue;
			//}

			// Distance culling
			// @todo(chaos): Cull against the projected convex hull, not just the outward face 
			// (we can store the most-distance vertex for each face with the convex to avoid actually having to project)
			if (Utilities::SignedSquare(DMin) > CullDistanceSq)
			{
				// Separated by more than CullDistance
				return false;
			}

			if (DMin > ConvexPlaneDMin)
			{
				ConvexPlaneN = ConN;
				ConvexPlaneX = ConX;
				ConvexPlaneDMin = DMin;
				ConvexPlaneIndexMin = PlaneIndex;
				ConvexPlaneTriangleIndexMin = IndexMin;
			}
		}

		//
		// SAT: Convex edges vs triangle edges
		//

		// Calculate the distance of triangle each edge to the convex separating plane
		FReal TriVertexConvexDMin0 = FVec3::DotProduct(Triangle.GetVertex(0) - ConvexPlaneX, ConvexPlaneN);
		FReal TriVertexConvexDMin1 = FVec3::DotProduct(Triangle.GetVertex(1) - ConvexPlaneX, ConvexPlaneN);
		FReal TriVertexConvexDMin2 = FVec3::DotProduct(Triangle.GetVertex(2) - ConvexPlaneX, ConvexPlaneN);
		FReal TriEdgeConvexDMin[3] =
		{
			FMath::Min(TriVertexConvexDMin2, TriVertexConvexDMin0),
			FMath::Min(TriVertexConvexDMin0, TriVertexConvexDMin1),
			FMath::Min(TriVertexConvexDMin1, TriVertexConvexDMin2),
		};

		FReal ConvexWinding = Convex.GetWindingOrder();
		FVec3 EdgeEdgeN = FVec3(0);
		FReal EdgeEdgeDMin = InvalidPhi;
		int32 ConvexEdgeIndexMin = INDEX_NONE;
		int32 TriEdgeIndexMin = INDEX_NONE;
		for (int32 ConvexEdgeLoopIndex = 0; ConvexEdgeLoopIndex < Convex.NumEdges(); ++ConvexEdgeLoopIndex)
		{
			// Handle reverse winding for negative scaled convexes. Loop over edges in reverse order, and reverse edge vertex order
			const int32 ConvexEdgeIndex = (ConvexWinding >= 0) ? ConvexEdgeLoopIndex : Convex.NumEdges() - ConvexEdgeLoopIndex - 1;
			const int32 ConvexEdgeVIndex0 = (ConvexWinding >= 0) ? 0 : 1;
			const int32 ConvexEdgeVIndex1 = (ConvexWinding >= 0) ? 1 : 0;

			// Skip convex edges beyond CullDistance of the triangle face
			const int32 ConvexEdgeVertexIndex0 = Convex.GetEdgeVertex(ConvexEdgeIndex, ConvexEdgeVIndex0);
			const int32 ConvexEdgeVertexIndex1 = Convex.GetEdgeVertex(ConvexEdgeIndex, ConvexEdgeVIndex1);
			const FReal FaceConvexD0 = ConvexVertexDs[ConvexEdgeVertexIndex0];
			const FReal FaceConvexD1 = ConvexVertexDs[ConvexEdgeVertexIndex1];
			if ((Utilities::SignedSquare(FaceConvexD0) > CullDistanceSq) && (Utilities::SignedSquare(FaceConvexD1) > CullDistanceSq))
			{
				continue;
			}

			// Convex edge vertices
			const FVec3 ConvexEdgeV0 = Convex.GetVertex(ConvexEdgeVertexIndex0);
			const FVec3 ConvexEdgeV1 = Convex.GetVertex(ConvexEdgeVertexIndex1);

			// Convex planes that form the edge
			const int32 ConvexEdgePlaneIndexA = Convex.GetEdgePlane(ConvexEdgeIndex, 0);
			const int32 ConvexEdgePlaneIndexB = Convex.GetEdgePlane(ConvexEdgeIndex, 1);
			const FVec3 ConvexEdgePlaneNormalA = Convex.GetPlane(ConvexEdgePlaneIndexA).Normal();
			const FVec3 ConvexEdgePlaneNormalB = Convex.GetPlane(ConvexEdgePlaneIndexB).Normal();

			for (int32 TriEdgeIndex = 0; TriEdgeIndex < 3; ++TriEdgeIndex)
			{
				// Skip triangle edges beyond cull distance of the convex separating face
				if (Utilities::SignedSquare(TriEdgeConvexDMin[TriEdgeIndex]) > CullDistanceSq)
				{
					continue;
				}

				// Triangle edge vertices
				const FVec3& TriEdgeV0 = Triangle.GetVertex(TriEdgeIndex);
				const FVec3& TriEdgeV1 = (TriEdgeIndex == 2) ? Triangle.GetVertex(0) : Triangle.GetVertex(TriEdgeIndex + 1);

				// Skip edge pairs that do not contribute to the Minkowski Sum surface
				// NOTE: This relies on the ordering of the edge planes from above. 
				// I.e., we require Sign(ConvexEdgePlaneNormalA x ConvexEdgePlaneNormalB) == Sign(ConvexEdgeV1 - ConvexEdgeV0)
				// Also note that we must pass the negated triangle normal in
				if (!IsOnMinkowskiSumConvexTriangle(ConvexEdgePlaneNormalA, ConvexEdgePlaneNormalB, ConvexEdgeV1 - ConvexEdgeV0, -TriN, TriEdgeV1 - TriEdgeV0))
				{
					continue;
				}

				// Separating axis
				// NOTE: Not normalized at this stage. We perform the projection against the
				// non-normalized axis and defer the square root until we know we need it
				FVec3 Axis = FVec3::CrossProduct(ConvexEdgeV1 - ConvexEdgeV0, TriEdgeV1 - TriEdgeV0);
				const FReal AxisLenSq = Axis.SizeSquared();

				// Pick consistent axis direction: away from triangle (we want a signed distance)
				const FReal Sign = FVec3::DotProduct(TriEdgeV0 - TriC, Axis);
				if (Sign < FReal(0))
				{
					Axis = -Axis;
				}

				// Ignore backface edges
				//const FReal AxisDotNormal = FVec3::DotProduct(TriN, Axis);
				//if (AxisDotNormal < -NormalTolerance)
				//{
				//	continue;
				//}

				const FReal ScaledSeparation = FVec3::DotProduct(ConvexEdgeV0 - TriEdgeV0, Axis);

				// Check cull distance on projected segments
				// Comparing square distances scaled by axis length to defer square root (keep the sign)
				const FReal ScaledSeparationSq = ScaledSeparation * FMath::Abs(ScaledSeparation);
				const FReal ScaledCullDistanceSq = CullDistanceSq * AxisLenSq;
				if (ScaledSeparationSq > ScaledCullDistanceSq)
				{
					return false;
				}

				// Comparing square distances scaled by axis length to defer square root (keep the sign)
				const FReal ScaledEdgeEdgeDMinSq = EdgeEdgeDMin * FMath::Abs(EdgeEdgeDMin) * AxisLenSq;
				if (ScaledSeparationSq > ScaledEdgeEdgeDMinSq)
				{
					// Now we need to know the actual separation and axis
					const FReal AxisInvLen = FMath::InvSqrt(AxisLenSq);
					EdgeEdgeDMin = ScaledSeparation * AxisInvLen;
					EdgeEdgeN = Axis * AxisInvLen;
					ConvexEdgeIndexMin = ConvexEdgeIndex;
					TriEdgeIndexMin = TriEdgeIndex;
				}
			}
		}

		// Determine which of the features we want to use ad fill in the output
		// NOTE: we rely on the fact that all valid Phi values are greater than InvalidPhi here

		const FReal TriFaceBias = FReal(1.e-2);	// Prevent flip-flop on near-parallel cases
		if ((TriPlaneDMin != InvalidPhi) && (TriPlaneDMin + TriFaceBias > ConvexPlaneDMin) && (TriPlaneDMin + TriFaceBias > EdgeEdgeDMin))
		{
			// Triangle face contact. The triangle normal is the separating axis
			const FReal SeparatingDistance = TriPlaneDMin;
			FVec3 SeparatingAxis = TriN;

			const FVec3 ConvexContactPoint = Convex.GetVertex(ConvexVertexIndexMin);
			const FVec3 TriangleContactPoint = ConvexContactPoint - SeparatingAxis * SeparatingDistance;
			OutContactPoint.ShapeContactPoints[0] = ConvexContactPoint;
			OutContactPoint.ShapeContactPoints[1] = TriangleContactPoint;
			OutContactPoint.ShapeContactNormal = SeparatingAxis;
			OutContactPoint.Phi = SeparatingDistance;
			return true;
		}

		if ((ConvexPlaneDMin != InvalidPhi) && (ConvexPlaneDMin > EdgeEdgeDMin))
		{
			// Convex face contact. The convex face is the separating axis, but it must point from the triangle to the convex
			const FReal SeparatingDistance = ConvexPlaneDMin;
			FVec3 SeparatingAxis = -ConvexPlaneN;

			const FVec3 TriangleContactPoint = Triangle.GetVertex(ConvexPlaneTriangleIndexMin);
			const FVec3 ConvexContactPoint = TriangleContactPoint + SeparatingAxis * SeparatingDistance;
			OutContactPoint.ShapeContactPoints[0] = ConvexContactPoint;
			OutContactPoint.ShapeContactPoints[1] = TriangleContactPoint;
			OutContactPoint.ShapeContactNormal = SeparatingAxis;
			OutContactPoint.Phi = SeparatingDistance;
			return true;
		}

		if (EdgeEdgeDMin != InvalidPhi)
		{
			// Edge-edge contact
			// The separating axis must point from the triangle to the convex
			const FReal SeparatingDistance = EdgeEdgeDMin;
			FVec3 SeparatingAxis = EdgeEdgeN;

			// Convex edge vertices
			const int32 ConvexEdgeVIndex0 = (ConvexWinding >= 0) ? 0 : 1;
			const int32 ConvexEdgeVIndex1 = (ConvexWinding >= 0) ? 1 : 0;
			const int32 ConvexEdgeVertexIndex0 = Convex.GetEdgeVertex(ConvexEdgeIndexMin, ConvexEdgeVIndex0);
			const int32 ConvexEdgeVertexIndex1 = Convex.GetEdgeVertex(ConvexEdgeIndexMin, ConvexEdgeVIndex1);
			const FVec3 ConvexEdgeV0 = Convex.GetVertex(ConvexEdgeVertexIndex0);
			const FVec3 ConvexEdgeV1 = Convex.GetVertex(ConvexEdgeVertexIndex1);

			// Triangle edge vertices
			const FVec3& TriEdgeV0 = Triangle.GetVertex(TriEdgeIndexMin);
			const FVec3& TriEdgeV1 = (TriEdgeIndexMin == 2) ? Triangle.GetVertex(0) : Triangle.GetVertex(TriEdgeIndexMin + 1);

			FReal ConvexEdgeT, TriEdgeT;
			FVec3 ConvexContactPoint, TriangleContactPoint;
			Utilities::NearestPointsOnLineSegments(ConvexEdgeV0, ConvexEdgeV1, TriEdgeV0, TriEdgeV1, ConvexEdgeT, TriEdgeT, ConvexContactPoint, TriangleContactPoint);

			OutContactPoint.ShapeContactPoints[0] = ConvexContactPoint;
			OutContactPoint.ShapeContactPoints[1] = TriangleContactPoint;
			OutContactPoint.ShapeContactNormal = SeparatingAxis;
			OutContactPoint.Phi = SeparatingDistance;
			return true;
		}

		// No valid features (should not happen - at least TriPlaneDMin should always be valid)
		ensureMsgf(false, TEXT("SATConvexTriangle failed to select a feature"));
		return false;
	}
}