// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/ParticleHandleFwd.h"
#include "Chaos/Collision/SimSweep.h"

namespace Chaos::Filter
{
	struct FQueryFilterData;
}

namespace Chaos::Private
{
	/**
	* Raycast input
	*/
	class FSimRaycastRay
	{
	public:
		FVec3 StartPos;
		FVec3 Dir;
		FReal Length;
	};

	/**
	* Raycast output
	*/
	using FSimRaycastHit = FSimSweepParticleHit;

	/*
	* Standard particle filter for use with Sim Raycasts that uses FQueryFilterData to specify what to exclude.
	* @todo(chaos): Currently accepts all.
	*/
	class FSimQueryDataParticleFilter
	{
	public:
		CHAOS_API FSimQueryDataParticleFilter(const Filter::FQueryFilterData& InFilterData);

		bool operator()(const FGeometryParticleHandle* InParticleHandle) const
		{
			return ApplyFilter(InParticleHandle);
		}

	private:
		CHAOS_API bool ApplyFilter(const FGeometryParticleHandle* InParticleHandle) const;

		Filter::FQueryFilterData FilterData;
	};

	/*
	* Standard shape filter for use with Sim Raycasts that uses FQueryFilterData to specify what to exclude.
	*/
	class FSimQueryDataShapeFilter
	{
	public:
		CHAOS_API FSimQueryDataShapeFilter(const Filter::FQueryFilterData& InFilterData, bool bInDetectSimple, bool bInDetectComplex);

		bool operator()(const FPerShapeData* InShape, const FImplicitObject* InImplicit) const
		{
			return ApplyFilter(InShape, InImplicit);
		}

	private:
		CHAOS_API bool ApplyFilter(const FPerShapeData* InShape, const FImplicitObject* InImplicit) const;

		union FFlags
		{
			FFlags() : Bits(0) {}
			struct
			{
				uint8 bDetectSimple : 1;
				uint8 bDetectComplex : 1;
			};
			uint8 Bits;
		};

		Filter::FQueryFilterData FilterData;
		FFlags Flags;
	};

	/**
	* Cast a ray against an implicit object with the specified transform.
	* @param InRay The ray info
	* @param InImplicit The implicit object to raycast against
	* @param InImplicitTransform The world space transform of the implicit object
	* @param InOutHit The raycast output. If InOutHit alreday contains a valid hit, then it will be left unchanged if the new hit is not before it
	* @return true if the ray hits the implicit object, false otherwise
	*/
	extern CHAOS_API bool SimRaycastImplicit(const FSimRaycastRay& InRay, const FImplicitObject* InImplicit, const FRigidTransform3& InImplicitTransform, FSimRaycastHit& InOutHit);

	/**
	* Cast a set of rays. This is intended to be used as an example - see the implementation for
	* how to write your own raycasts. This version only visits the SpatialAcceleration once
	* using the aggregate bounds of all of the rays in InRays. It is well suited to running
	* multiple closely located short raycasts, but will likely underperform for long off-axis raycasts
	* because of the way it collects objects in the raycast bounds.
	* 
	* @param InRays The batch of rays to cast. They should be fairly close together with a "reasonable" bounds for perf reasons.
	* @param QueryFilterData The filter to reject shapes.
	* @param bDetectSimple Whether to detect simple collision shapes.
	* @param bDetectComplex Whether to detect complex collision shapes.
	* @param OutHits The results of the raycasts. Must contain one and only one entry per input ray.
	* @return true is any of the rays hit something, false otherwise.
	*/
	extern CHAOS_API bool SimRaycastBatchFirstHits(
		ISpatialAcceleration<FAccelerationStructureHandle, FReal, 3>* SpatialAcceleration,
		const TArrayView<const Private::FSimRaycastRay>& InRays,
		const Filter::FQueryFilterData& QueryFilterData,
		const bool bDetectSimple, const bool bDetectComplex,
		const TArrayView<Private::FSimRaycastHit>& OutHits);

}
