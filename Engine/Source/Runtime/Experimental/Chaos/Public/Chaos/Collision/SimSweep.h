// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Collision/CollisionFilter.h"
#include "Chaos/GeometryQueries.h"
#include "Chaos/ISpatialAcceleration.h"
#include "Chaos/ParticleHandle.h"

namespace Chaos
{
	namespace Private
	{
		/**
		* Results from a SimSweepParticle.
		* NOTE: Only HitParticle is guaranteed to be initialized. If HitParticle is null, all other properties are undefined.
		*/
		class FSimSweepParticleHit
		{
		public:
			FSimSweepParticleHit()
				: HitParticle(nullptr)
			{
			}

			void Init()
			{
				HitParticle = nullptr;
			}

			bool IsHit() const
			{
				return HitParticle != nullptr;
			}

			// The particle that we hit, or null
			const FGeometryParticleHandle* HitParticle;

			// The shape that we hit (or uninitialized)
			const FPerShapeData* HitShape;

			// The geometry that we hit (or uninitialized)
			const FImplicitObject* HitGeometry;

			// The contact position (or uninitialized)
			FVec3 HitPosition;

			// The contact normal for HitDistance >= 0, or MTD direction for HitDistance < 0 (or uninitialized)
			FVec3 HitNormal;

			// The normal of the face we hit, if HitFaceIndex is set (or uninitialized)
			FVec3 HitFaceNormal;

			// The distance along the sweep at the hit, or the seperation if negative (i.e., -Penetration) (or uninitialized)
			FReal HitDistance;

			// Time of impact [0,1] (or uninitialized)
			FReal HitTOI;

			// The face index of the shape we hit, or INDEX_NONE (or uninitialized)
			int32 HitFaceIndex;
		};

		/**
		* Results from SimOverlapBounds test
		*/
		class FSimOverlapParticleShape
		{
		public:
			FSimOverlapParticleShape()
			{
			}

			// The particle that we hit, or null
			const FGeometryParticleHandle* HitParticle;

			// The shape that we hit (or uninitialized)
			const FPerShapeData* HitShape;

			// The world-space transform of the geometry that was hit
			FRigidTransform3 ShapeWorldTransform;
		};

		/**
		* A spatial acceleration visitor that forwards callbacks to a functor
		*/
		template<typename TVisitor>
		class TSimSweepSQVisitor : public ISpatialVisitor<FAccelerationStructureHandle, FReal>
		{
		public:
			using FVisitorData = TSpatialVisitorData<FAccelerationStructureHandle>;

			TSimSweepSQVisitor(const TVisitor& InVisitor)
				: Visitor(InVisitor)
			{
			}

			virtual bool Overlap(const FVisitorData& Instance) override final
			{
				return CallVisitor(Instance);
			}

			virtual bool Sweep(const FVisitorData& Instance, FQueryFastData& CurData) override final
			{
				return CallVisitor(Instance);
			}

			virtual bool Raycast(const FVisitorData& Instance, FQueryFastData& CurData) override final
			{
				return CallVisitor(Instance);
			}
		private:
			bool CallVisitor(const FVisitorData& Instance)
			{
				FGeometryParticleHandle* Particle = Instance.Payload.GetGeometryParticleHandle_PhysicsThread();
				Visitor(Particle);
				return true;
			}

			const TVisitor& Visitor;
		};

		/**
		* Produce a set of particles and their shapes that overlap the query bounds and pass the particle and shape filters.
		* 
		* @TParam TParticleFilter a particle that returns true to check a particle for overlaps. Signature bool(const FGeometryParticleHandle*)
		* 
		* @TParam TShapeFilter a shape that returns true to check a particle for overlaps. Signature bool(const FPerShapeData*, const FImplicitObject*)
		*
		* @TParam TOverlapCollector an overlap collector functor with the signature void(const FSimOverlapParticleShape&)
		*/
		template<typename TParticleFilter, typename TShapeFilter, typename TOverlapCollector>
		void SimOverlapBounds(
			ISpatialAcceleration<FAccelerationStructureHandle, FReal, 3>* SpatialAcceleration,
			const FAABB3& QueryBounds,
			TParticleFilter& ParticleFilter,
			TShapeFilter& ShapeFilter,
			TOverlapCollector& OverlapCollector)
		{
			FImplicitBox3 QueryBox(QueryBounds.Min(), QueryBounds.Max());

			// Functor: Process a particle encountered by the overla through the spatial acceleration
			const auto& ProcessParticle =
				[&QueryBox, &ParticleFilter, &ShapeFilter, &OverlapCollector]
				(const FGeometryParticleHandle* OtherParticle)
				-> void
			{
				// Check the particle filter to see if we should process this particle pair
				if (ParticleFilter(OtherParticle))
				{
					// The particle transform
					const FRigidTransform3 OtherWorldTransform = OtherParticle->GetTransformXR();
					for (const TUniquePtr<FPerShapeData>& OtherShape : OtherParticle->ShapesArray())
					{
						const FImplicitObject* OtherImplicit = OtherShape->GetLeafGeometry();

						// Check the filter to see if we want to  process this shape pair
						const bool bOverlapShape = ShapeFilter(OtherShape.Get(), OtherImplicit);

						if (bOverlapShape)
						{
							const FRigidTransform3 OtherShapeWorldTransform = FRigidTransform3(OtherShape->GetLeafRelativeTransform()) * OtherWorldTransform;

							const bool bOverlapHit = OverlapQuery(
								*OtherImplicit,
								OtherShapeWorldTransform,
								QueryBox,
								FRigidTransform3::Identity,
								0/*Thickness*/,
								nullptr);

							// If we have a hit, pass it to the collector
							if (bOverlapHit)
							{
								FSimOverlapParticleShape Overlap;
								Overlap.HitParticle = OtherParticle;
								Overlap.HitShape = OtherShape.Get();
								Overlap.ShapeWorldTransform = OtherShapeWorldTransform;

								OverlapCollector(Overlap);
							}
						}
					}
				}
			};

			// Generate the set of objects that overlap the bounds
			TSimSweepSQVisitor SpatialAcclerationVisitor(ProcessParticle);
			SpatialAcceleration->Overlap(QueryBounds, SpatialAcclerationVisitor);
		}

		/**
		* Sweep SweptParticle against OtherParticle. The provided filter policy can be used to reject shape pairs.
		* The collector policy chooses which sweep results(s) to keep.
		*
		* @tparam TShapeFilter A custom filter functor with the signature bool(const FPerShapeData* SweptShape, const FImplicitObject* SweptImplicit, const FPerShapeData* OtherShape, const FImplicitObject* OtherImplicit)
		* used to accept/reject shapes pairs. It should return true if the pair should be considered for a sweep check.
		* @see FSimSweepShapeFilterNarrowPhase which provides the standard narrowphase filter used in collision detection.
		*
		* @tparam THitCollector A hit collector functor with signature bool(const FVec3& Dir, const FReal Length, const FSimSweepParticleHit& InHit).
		* It should return true to continue checking further objects, or false to stop any further sweeping.
		* @see FSimSweepCollectorFirstHit which keeps the hit with the lowest time of impact and the most opposing normal if there is a tie.
		*
		* @param SweptParticle The particle to be swept
		* @param OtherParticle The particle to be swept against
		* @param StartPos The start position of the particle to be swept
		* @param Rot The rotation to use for the sweep
		* @param Dir The direction of the sweep (must be normalized)
		* @param Length The sweep length
		* @param ShapeFilter The filter to use to accept/reject shape pairs within the particle pairs
		* @param HitCollector Hits are passed to the collector one at a time in a random order (i.e., they are not sorted by TOI)
		* 
		* @note This function does not reset the HitCollector.
		*/
		template<typename TShapeFilter, typename THitCollector>
		void SimSweepParticlePair(
			const FGeometryParticleHandle* SweptParticle,
			const FGeometryParticleHandle* OtherParticle,
			const FVec3& StartPos,
			const FRotation3& Rot,
			const FVec3& Dir,
			const FReal Length,
			TShapeFilter& ShapeFilter,
			THitCollector& HitCollector)
		{
			// The particle transforms
			const FRigidTransform3 SweptWorldTransform = FRigidTransform3(StartPos, Rot);
			const FRigidTransform3 OtherWorldTransform = OtherParticle->GetTransformXR();

			// Storage for the next hit result in the sweep (we only need one at a time)
			FSimSweepParticleHit Hit;
			Hit.HitParticle = OtherParticle;

			// Visit all shape pairs and run a sweep (if they pass a filter check)
			for (const TUniquePtr<FPerShapeData>& SweptShape : SweptParticle->ShapesArray())
			{
				const FImplicitObject* SweptImplicit = SweptShape->GetLeafGeometry();
				const FRigidTransform3 SweptShapeWorldTransform = FRigidTransform3(SweptShape->GetLeafRelativeTransform()) * SweptWorldTransform;

				for (const TUniquePtr<FPerShapeData>& OtherShape : OtherParticle->ShapesArray())
				{
					const FImplicitObject* OtherImplicit = OtherShape->GetLeafGeometry();

					// Check the filter to see if we want to  process this shape pair
					const bool bSweepShape = ShapeFilter(SweptShape.Get(), SweptImplicit, OtherShape.Get(), OtherImplicit);

					if (bSweepShape)
					{
						const FRigidTransform3 OtherShapeWorldTransform = FRigidTransform3(OtherShape->GetLeafRelativeTransform()) * OtherWorldTransform;

						const bool bSweepHit = SweepQuery(
							*OtherImplicit,
							OtherShapeWorldTransform,
							*SweptImplicit,
							SweptShapeWorldTransform,
							Dir,
							Length,
							Hit.HitDistance,
							Hit.HitPosition,
							Hit.HitNormal,
							Hit.HitFaceIndex,
							Hit.HitFaceNormal,
							0/*Thickness*/,
							true/*bComputeMTD*/);

						// If we have a hit, pass it to the collector
						if (bSweepHit)
						{
							// Fill in the hit data that wasn't filled in by SweepQuery
							Hit.HitTOI = (Hit.HitDistance > 0) ? (Hit.HitDistance / Length) : 0;

							HitCollector(Dir, Length, Hit);
						}
					}
				}
			}
		}

		/**
		* Sweep a particle against all other particles that it may hit. The particles are visited in a semi-random order, depending on how
		* the particles are held in the spatial acceleration. The provided filter policies can be used to reject particle pairs or shape pairs.
		* The collector policy chooses which sweep result(s) to keep.
		* 
		* @tparam TParticleFilter A custom filter functor with the signature bool(const FGeometryParticleHandle* SweptParticle, const FGeometryParticleHandle* OtherParticle)
		* used to accept/reject particle pairs. It should return true if the particle pair should be considered for a sweep check.
		* @see FSimSweepParticleFilterBroadPhase which provides the standard broadphase filter used in collision detection.
		*
		* @tparam TShapeFilter A custom filter functor with the signature bool(const FPerShapeData* SweptShape, const FImplicitObject* SweptImplicit, const FPerShapeData* OtherShape, const FImplicitObject* OtherImplicit)
		* used to accept/reject shapes pairs. It should return true if the pair should be considered for a sweep check.
		* @see FSimSweepShapeFilterNarrowPhase which provides the standard narrowphase filter used in collision detection.
		*
		* @tparam THitCollector A hit collector functor with signature bool(const FVec3& Dir, const FReal Length, const FSimSweepParticleHit& InHit). 
		* It should return true to continue checking further objects, or false to stop any further sweeping. 
		* @see FSimSweepCollectorFirstHit which keeps the hit with the lowest time of impact and the most opposing normal if there is a tie.
		* 
		* @param SweptParticle The particle to be swept
		* @param StartPos The start position of the particle to be swept
		* @param Rot The rotation to use for the sweep
		* @param Dir The direction of the sweep (must be normalized)
		* @param Length The sweep length
		* @param ParticleFilter The filter to use to accept/reject particle pairs
		* @param ShapeFilter The filter to use to accept/reject shape pairs within the particle pairs
		* @param HitCollector Hits are passed to the collector one at a time in a semi-random order (i.e., they are not sorted by TOI)
		*/
		template<typename TParticleFilter, typename TShapeFilter, typename THitCollector>
		void SimSweepParticle(
			ISpatialAcceleration<FAccelerationStructureHandle, FReal, 3>* SpatialAcceleration,
			const FGeometryParticleHandle* SweptParticle,
			const FVec3& StartPos, 
			const FRotation3& Rot,
			const FVec3& Dir,
			const FReal Length,
			TParticleFilter& ParticleFilter, 
			TShapeFilter& ShapeFilter,
			THitCollector& HitCollector)
		{
			// Cannot sweep infinite particles (they are all static anyway)
			if (!SweptParticle->HasBounds())
			{
				return;
			}

			// @todo(chaos): do we need to generate a hit for no sweep?
			if (Length < UE_SMALL_NUMBER)
			{
				return;
			}

			// Functor: Process a particle encountered by the sweep through the spatial acceleration
			// by sweeping all shapes of the swept particle against all shapes of the other particle.
			const auto& ProcessParticlePair = 
				[SweptParticle, &StartPos, &Dir, &Length, &Rot, &ParticleFilter, &ShapeFilter, &HitCollector]
				(const FGeometryParticleHandle* OtherParticle)
				-> void
				{
					// We cannot hit ourself
					if (OtherParticle != SweptParticle)
					{
						// Check the particle filter to see if we should process this particle pair
						if (ParticleFilter(SweptParticle, OtherParticle))
						{
							// Do the sweep
							SimSweepParticlePair(SweptParticle, OtherParticle, StartPos, Rot, Dir, Length, ShapeFilter, HitCollector);
						}
					}
				};

			// Calculate the ray start/end and extents for the spatial acceleration sweep
			// (the sweep through the spatial acceleration requires a ray centered on the bounds)
			const FAABB3 TransformedParticleBounds = SweptParticle->GetGeometry()->CalculateTransformedBounds(FRigidTransform3(StartPos, Rot));
			const FVec3 BoundsExtents = FReal(0.5) * TransformedParticleBounds.Extents();
			const FVec3 BoundsStartPos = TransformedParticleBounds.Center();

			// Sweep our bounds through the spatial acceleration structure and then sweep the shapes of our particle
			// against those of any particles that our bounds sweep encounters.
			TSimSweepSQVisitor SpatialAcclerationVisitor(ProcessParticlePair);
			SpatialAcceleration->Sweep(BoundsStartPos, Dir, Length, BoundsExtents, SpatialAcclerationVisitor);
		}


		/**
		* The default particle pair filter for sim sweeps that uses the same test as in the broadphase.
		* I.e., sweep will be stopped by any object that the swept particle can collide with.
		*/
		class FSimSweepParticleFilterBroadPhase
		{
		public:
			FSimSweepParticleFilterBroadPhase(FIgnoreCollisionManager* InIgnoreCollisionManager)
				: IgnoreCollisionManager(InIgnoreCollisionManager)
			{
			}

			bool operator()(const FGeometryParticleHandle* SweptParticle, const FGeometryParticleHandle* OtherParticle)
			{
				return ParticlePairBroadPhaseFilter(SweptParticle, OtherParticle, IgnoreCollisionManager);
			}

		private:
			FIgnoreCollisionManager* IgnoreCollisionManager;
		};

		/**
		* The default shape pair filter for sim sweeps that uses the same test as in the narrowphase.
		* I.e., sweep will be stopped by any object that the swept particle can collide with.
		*/
		class FSimSweepShapeFilterNarrowPhase
		{
		public:
			bool operator()(const FPerShapeData* SweptShape, const FImplicitObject* SweptImplicit, const FPerShapeData* OtherShape, const FImplicitObject* OtherImplicit)
			{
				const EImplicitObjectType SweptImplicitType = SweptImplicit ? GetInnerType(SweptImplicit->GetType()) : ImplicitObjectType::Unknown;
				const EImplicitObjectType OtherImplicitType = OtherImplicit ? GetInnerType(OtherImplicit->GetType()) : ImplicitObjectType::Unknown;

				return ShapePairNarrowPhaseFilter(SweptImplicitType, SweptShape, OtherImplicitType, OtherShape);
			}
		};

		/**
		* A hit collector for SimSweepParticle that just keeps the first hit, except for initial overlaps where the
		* sweep is already moving us away from the contact. When there are multiple hits at the same TOI/initial overlap
		* we take the hit with the most opposing normal.
		*/
		class FSimSweepCollectorFirstHit
		{
		public:
			/**
			* @param InHitDistanceEqualTolerance Sweep hits within this distance are assumed to be at equal distance, in which 
			* case we take the hit with the most-opposing normal
			* 
			* @param OutHit a reference to a hit structure that will contain the results.
			*/
			FSimSweepCollectorFirstHit(const FReal InHitDistanceEqualTolerance, FSimSweepParticleHit& OutFirstHit)
				: FirstHit(OutFirstHit)
				, HitDistanceEqualTolerance(InHitDistanceEqualTolerance)
			{
				Init();
			}

			/**
			* Keep the first hit and continue looking for more hits
			*/
			bool operator()(const FVec3& Dir, const FReal Length, const FSimSweepParticleHit& InHit)
			{
				ProcessHit(Dir, Length, InHit);
				return true;
			}

			void Init()
			{
				FirstHit.Init();
			}

			bool IsHit() const
			{
				return FirstHit.IsHit();
			}

			const FSimSweepParticleHit& GetFirstHit() const
			{
				check(IsHit());
				return FirstHit;
			}

		private:
			void ProcessHit(const FVec3& Dir, const FReal Length, const FSimSweepParticleHit& InHit)
			{
				if (!FirstHit.IsHit() || (InHit.HitDistance < FirstHit.HitDistance + HitDistanceEqualTolerance))
				{
					// If the distance is very close to the existing hit, keep the one with the most opposing normal.
					if (FirstHit.IsHit() && FMath::IsNearlyEqual(InHit.HitDistance, FirstHit.HitDistance))
					{
						const FReal ExistingDotNormal = FVec3::DotProduct(Dir, FirstHit.HitNormal);
						const FReal NewDotNormal = FVec3::DotProduct(Dir, InHit.HitNormal);
						if (NewDotNormal < ExistingDotNormal)
						{
							FirstHit = InHit;
						}
						return;
					}

					// If this is a closer hit we will take it unless we are already moving out of contact.
					// NOTE: we will only ever get an outward pointing normal for initial overlaps (HitDistance <= 0).
					if ((InHit.HitDistance > 0) || (FVec3::DotProduct(InHit.HitNormal, Dir) < 0))
					{
						FirstHit = InHit;
						return;
					}
				}
			}

			FSimSweepParticleHit& FirstHit;
			FReal HitDistanceEqualTolerance;
		};

		/**
		* Sweep a particle to find the first impact with another particle that the particle would normally collide with 
		* according to the sim filter data on the particles.
		* 
		* @param SpatialAcceleration The spatial acceleration structure used in the broadphase
		* @param IgnoreCollisionManager The ignore collision manager used in the main broadphase (or null)
		* @param SweptParticle The particle to be swept
		* @param StartPos The start position of the particle to be swept
		* @param Rot The rotation to use for the sweep
		* @param Dir The direction of the sweep (must be normalized)
		* @param Length The sweep length
		* @param OutHit Output set to the first hit if there was one, otherwise invalidated.
		* 
		* @return true if there was a hit, otherwise false
		* 
		* @note This call SimSweepParticle with the FSimSweepParticleFilterCollidable and FSimSweepShapeFilterCollidable filter policies,
		* and the FSimSweepCollectorFirstHit hit collector. If these policies are not what you want, you can create your own policies 
		* and/or collector (e.g., to collect all hits) and use SimSweepParticle directly.
		* 
		*/
		extern bool SimSweepParticleFirstHit(
			ISpatialAcceleration<FAccelerationStructureHandle, FReal, 3>* SpatialAcceleration,
			FIgnoreCollisionManager* IgnoreCollisionManager,
			const FGeometryParticleHandle* SweptParticle,
			const FVec3& StartPos, 
			const FRotation3& Rot,
			const FVec3& Dir,
			const FReal Length,
			FSimSweepParticleHit& OutHit,
			const FReal InHitDistanceEqualTolerance = UE_KINDA_SMALL_NUMBER);

		/**
		* Collect all the shapes that overlap a bounding box.
		* @todo(chaos): Add a few more options like Complex vs Simple collection
		*/
		extern bool SimOverlapBoundsAll(
			ISpatialAcceleration<FAccelerationStructureHandle, FReal, 3>* SpatialAcceleration,
			const FAABB3& QueryBounds,
			TArray<FSimOverlapParticleShape>& Overlaps);
	}

}