// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// @todo(chaos): Deprecate this header
// Moved to PBDCollisionContainerSolver.h

#include "Chaos/Collision/PBDCollisionContainerSolver.h"
