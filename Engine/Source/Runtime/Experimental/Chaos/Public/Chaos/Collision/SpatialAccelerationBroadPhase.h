// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Collision/CollisionConstraintAllocator.h"
#include "Chaos/Collision/CollisionContext.h"
#include "Chaos/Collision/CollisionFilter.h"
#include "Chaos/Collision/StatsData.h"
#include "Chaos/ISpatialAccelerationCollection.h"
#include "Chaos/ParticleHandleFwd.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/PBDRigidsSOAs.h"
#include "Chaos/Capsule.h"
#include "ChaosStats.h"
#include "Chaos/EvolutionResimCache.h"
#include "Chaos/GeometryParticlesfwd.h"
#include "Chaos/AABBTree.h"
#include "ChaosVisualDebugger/ChaosVDContextProvider.h"
#include "Tasks/Task.h"

// Set to 1 to enable some sanity chacks on the filtered broadphase results
#ifndef CHAOS_CHECK_BROADPHASE
#define CHAOS_CHECK_BROADPHASE 0
#endif

namespace Chaos::CVars
{
	extern bool bChaosMidPhaseRedistributionEnabled;
	extern int32 ChaosOneWayInteractionPairCollisionMode;
	extern int32 NumWorkerCollisionFactor;
}

namespace Chaos
{
	template <typename TPayloadType, typename T, int d>
	class ISpatialAcceleration;
	class IResimCacheBase;

	namespace Private
	{

		/**
		 * Check whether the two particles are allowed to collide, and also whether we should flip their order.
		 */
		inline bool ParticlePairCollisionAllowed(
			const FGeometryParticleHandle* Particle1, 
			const FGeometryParticleHandle* Particle2, 
			const FIgnoreCollisionManager& IgnoreCollisionManager, 
			const bool bIsResimming,  
			bool& bOutSwapOrder)
		{
			bOutSwapOrder = false;

			if (!ParticlePairBroadPhaseFilter(Particle1, Particle2, &IgnoreCollisionManager))
			{
				return false;
			}

			bool bIsMovingKinematic1 = false;
			bool bIsDynamicAwake1 = false;
			bool bIsDynamicAsleep1 = false;
			const FPBDRigidParticleHandle* Rigid1 = Particle1->CastToRigidParticle();
			if (Rigid1 != nullptr)
			{
				bIsMovingKinematic1 = Rigid1->IsMovingKinematic();
				bIsDynamicAsleep1 = Rigid1->IsDynamic() && Rigid1->IsSleeping();
				bIsDynamicAwake1 = Rigid1->IsDynamic() && !Rigid1->IsSleeping();;
			}

			bool bIsMovingKinematic2 = false;
			bool bIsDynamicAwake2 = false;
			bool bIsDynamicAsleep2 = false;
			const FPBDRigidParticleHandle* Rigid2 = Particle2->CastToRigidParticle();
			if (Rigid2 != nullptr)
			{
				bIsMovingKinematic2 = Rigid2->IsMovingKinematic();
				bIsDynamicAsleep2 = Rigid2->IsDynamic() && Rigid2->IsSleeping();
				bIsDynamicAwake2 = Rigid2->IsDynamic() && !Rigid2->IsSleeping();
			}

			// Used to determine a winner in cases where we visit particle pairs in both orders
			const bool bIsParticle1Preferred = AreParticlesInPreferredOrder(Particle1, Particle2);

			bool bAcceptParticlePair = false;
			if (!bIsResimming)
			{
				// Assumptions for the non-resim case:
				//		- Particle1 is the particle from an outer loop over either
				//			(A) all dynamic particles, or 
				//			(B) all awake dynamic and moving kinematic particles.
				//		- Particle2 is one of the particles whose bounds overlaps Particle1

				// If the first particle is an awake dynamic
				//		- accept if the other particle is an awake dynamic and we are the preferred particle (lower ID)
				//		- accept if the other particle is in any other state (static, kinematic, asleep)
				if (bIsDynamicAwake1)
				{
					if (bIsDynamicAwake2)
					{
						bAcceptParticlePair = bIsParticle1Preferred;
					}
					else
					{
						bAcceptParticlePair = true;
					}
				}
				// If the first particle is an asleep dynamic
				//		- accept if the other particle is a moving kinematic
				//		- reject if the other particle is a stationary kinematic or static - sleeping and static particles do not collide
				//		- reject if the other particle is an awake dynamic - will be picked up when visiting in the opposite order
				//		- reject if the other particle if an asleep dynamic - two sleeping dynamics do not collide
				else if (bIsDynamicAsleep1)
				{
					bAcceptParticlePair = bIsMovingKinematic2;
				}
				// If the first particle is a moving kinematic
				//		- accept if the other particle is sleeping dynamic
				//		- reject if the other particle is an awake dynamic - will be picked up when visiting in the opposite order
				//		- reject if the other particle is a kinematic or static - they do not collide
				else if (bIsMovingKinematic1)
				{
					bAcceptParticlePair = bIsDynamicAsleep2;
				}
				// If the first particle is static or non-moving kinematic
				//		- reject always - will be picked up when visiting in the opposite order
				else
				{
				}
			}
			else
			{
				// When resimming we iterate over "desynced" particles which may be kinematic so:
				// - Particle1 is always desynced
				// - Particle2 may also be desynced, in which case we will also visit the opposite ordering regardless of dynamic/kinematic status
				// - Particle1 may be static, kinematic, dynamic, asleep
				// - Particle2 may be static, kinematic, dynamic, asleep
				// 
				// Even though Particle1 may be kinematic when resimming, we want to create the contacts in the original order (i.e., dynamic first)
				// 
				const bool bIsParticle2Desynced = bIsResimming && (Particle2->SyncState() == ESyncState::HardDesync);
				const bool bIsKinematic1 = !bIsDynamicAwake1 && !bIsDynamicAsleep1;
				const bool bIsKinematic2 = !bIsDynamicAwake2 && !bIsDynamicAsleep2;

				// If Particle1 is dynamic, accept if the other is asleep or nor dynamic
				if ((bIsDynamicAwake1 && !bIsDynamicAwake2) || (bIsDynamicAsleep1 && bIsKinematic2))
				{
					bAcceptParticlePair = true;
				}

				// If Particle1 is non dynamic but particle 2 is dynamic, the case should already be handled by (1) for
				// the desynced dynamic - synced/desynced (static,kinematic,asleep) pairs. But we still need to process
				// the desynced (static,kinematic,asleep) against the synced dynamic since this case have not been handled by (1)
				if (!bIsDynamicAwake1 && bIsDynamicAwake2)
				{
					bAcceptParticlePair = !bIsParticle2Desynced;
				}
				// If Particle1 and Particle2 are dynamic we validate the pair if particle1 has higher ID to discard duplicates since we will visit twice the pairs
				// We validate the pairs as well if the particle 2 is synced since  we will never visit the opposite order (we only iterate over desynced particles)
				else if (bIsDynamicAwake1 && bIsDynamicAwake2)
				{
					bAcceptParticlePair = bIsParticle1Preferred || !bIsParticle2Desynced;
				}
				// If Particle1 is kinematic we are discarding the pairs against sleeping desynced particles since the case has been handled by (2)
				else if (bIsKinematic1 && bIsDynamicAsleep2)
				{
					bAcceptParticlePair = !bIsParticle2Desynced;
				}
			}

			// Since particles can change type (between Kinematic and Dynamic) we may visit them in different orders at different times, but if we allow
			// that it would break Resim and constraint reuse. Also, if only one particle is Dynamic, we want it in first position. This isn't a strtct 
			// requirement but some downstream systems assume this is true (e.g., CCD, TriMesh collision).
			if (bAcceptParticlePair)
			{
				bOutSwapOrder = ShouldSwapParticleOrder((bIsDynamicAwake1 || bIsDynamicAsleep1), (bIsDynamicAwake2 || bIsDynamicAsleep2), bIsParticle1Preferred);
			}

			return bAcceptParticlePair;
		}

		/**
		 * Output element from the broadphase
		*/
		class FBroadPhaseOverlap
		{
		public:
			FBroadPhaseOverlap()
			{
			}

			FBroadPhaseOverlap(FGeometryParticleHandle* Particle0, FGeometryParticleHandle* Particle1, const int32 InSearchParticleIndex)
				: Particles{ Particle0, Particle1 }
				, SearchParticleIndex(InSearchParticleIndex)
				, bCollisionsEnabled(true)
			{
			}

			void ApplyFilter(const FIgnoreCollisionManager& IgnoreCollisionManager, const bool bIsResimming)
			{
				bool bSwapOrder = false;

				bCollisionsEnabled = Private::ParticlePairCollisionAllowed(Particles[0], Particles[1], IgnoreCollisionManager, bIsResimming, bSwapOrder);

				if (bSwapOrder)
				{
					Swap(Particles[0], Particles[1]);
					SearchParticleIndex = 1 - SearchParticleIndex;
				}
			}

			FGeometryParticleHandle* Particles[2];
			int32 SearchParticleIndex;
			bool bCollisionsEnabled;
		};

		/**
		 * Per-thread context for the broadphase.
		*/
		class FBroadPhaseContext
		{
		public:
			FBroadPhaseContext()
			{
			}

			void Reset()
			{
				Overlaps.Reset();
				MidPhases.Reset();
				CollisionContext.Reset();
			}

			TArray<FBroadPhaseOverlap> Overlaps;
			TArray<FParticlePairMidPhase*> MidPhases;
			FCollisionContext CollisionContext;
		};

		/**
		 * A visitor for the spatial partitioning system used to build the set of objects overlapping a bounding box.
		 */
		struct FSimOverlapVisitor
		{
			FSimOverlapVisitor(FGeometryParticleHandle* ParticleHandle, Private::FBroadPhaseContext& InContext)
				: Context(InContext)
				, ParticleUniqueIdx(ParticleHandle ? ParticleHandle->UniqueIdx() : FUniqueIdx(0))
				, AccelerationHandle(ParticleHandle)
			{
			}

			UE_DEPRECATED(5.7, "Use the constructor that does not take the sim filter.")
			FSimOverlapVisitor(FGeometryParticleHandle* ParticleHandle, const FCollisionFilterData& InSimFilterData, Private::FBroadPhaseContext& InContext)
				: FSimOverlapVisitor(ParticleHandle, InContext)
			{
			}

			bool VisitOverlap(const TSpatialVisitorData<FAccelerationStructureHandle>& Instance)
			{
				FGeometryParticleHandle* Particle1 = AccelerationHandle.GetGeometryParticleHandle_PhysicsThread();
				FGeometryParticleHandle* Particle2 = Instance.Payload.GetGeometryParticleHandle_PhysicsThread();
			
				Context.Overlaps.Emplace(Particle1, Particle2, 0);

				return true;
			}

			bool VisitSweep(TSpatialVisitorData<FAccelerationStructureHandle>, FQueryFastData& CurData)
			{
				check(false);
				return false;
			}

			bool VisitRaycast(TSpatialVisitorData<FAccelerationStructureHandle>, FQueryFastData& CurData)
			{
				check(false);
				return false;
			}

			UE_DEPRECATED(5.8, "This has been deprecated in favor of PrePreFilter")
			const void* GetQueryData() const { return nullptr; }

			UE_DEPRECATED(5.8, "This has been deprecated in favor of PrePreFilter")
			const void* GetSimData() const { return nullptr; }

			bool ShouldIgnore(const TSpatialVisitorData<FAccelerationStructureHandle>& Instance) const
			{
				return Instance.Payload.UniqueIdx() == ParticleUniqueIdx;
			}
			/** Return a pointer to the payload on which we are querying the acceleration structure */
			const void* GetQueryPayload() const
			{
				return &AccelerationHandle;
			}

			bool HasBlockingHit() const
			{
				return false;
			}

			bool PrePreFilter(const FAccelerationStructureHandle& Payload) const
			{
				if (ShouldIgnore(Payload))
				{
					return true;
				}
				return Payload.PrePreFilter(AccelerationHandle);
			}

		private:
			Private::FBroadPhaseContext& Context;
			FUniqueIdx ParticleUniqueIdx; // unique id of the particle visiting, used to skip self intersection as early as possible

			/** Handle to be stored to retrieve the payload on which we are querying the acceleration structure*/
			FAccelerationStructureHandle AccelerationHandle;
		};
	}

	/**
	 * A broad phase that iterates over particle and uses a spatial acceleration structure to output
	 * potentially overlapping SpatialAccelerationHandles.
	 */
	class FSpatialAccelerationBroadPhase
	{
	public:
		using FAccelerationStructure = ISpatialAcceleration<FAccelerationStructureHandle, FReal, 3>;

		FSpatialAccelerationBroadPhase(const FPBDRigidsSOAs& InParticles)
			: Particles(InParticles)
			, SpatialAcceleration(nullptr)
			, NumActiveBroadphaseContexts(0)
			, bNeedsResim(false)
		{
		}

		const TArray<UE::Tasks::FTask>& GetBroadToNarrowTasks() const;

		void SetSpatialAcceleration(const FAccelerationStructure* InSpatialAcceleration)
		{
			SpatialAcceleration = InSpatialAcceleration;
		}

		/**
		 * Generate all overlapping pairs and spawn a midphase object to handle collisions for each of them
		 */
		void ProduceOverlaps(
			FReal Dt,
			Private::FCollisionConstraintAllocator* Allocator,
			const FCollisionDetectorSettings& Settings,
			IResimCacheBase* ResimCache);

		/**
		 * Generate all the collision constraints for the set of overlapping objects produced by the broad phase
		*/
		void ProduceCollisions(FReal Dt);

		void GatherConstraints(bool bIsDeterministic)
		{
			if (bSingleWorkerPhysics)
			{
				if (bIsDeterministic)
				{
					QUICK_SCOPE_CYCLE_COUNTER(STAT_SortConstraint);
					BroadphaseContexts[0].CollisionContext.Allocator->NewActiveConstraints.Sort(
						[](const FPBDCollisionConstraintHandle& L, const FPBDCollisionConstraintHandle& R)
						{
							return L.GetContact().GetCollisionSortKey() < R.GetContact().GetCollisionSortKey();
						});
				}
			}
			else
			{
				// This lambda merge constraints of 2 broad phase context. 
				// If it is deterministic the lambda assume they are already sorted and merge them in the first given context 
				// maintaining the sorted data.
				auto MergeLambda = [this, bIsDeterministic](int32 SmallIndexToMerge, int32 BigIndexToMerge)
					{
						QUICK_SCOPE_CYCLE_COUNTER(STAT_MergeConstraints);
						Private::FBroadPhaseContext& Context = BroadphaseContexts[SmallIndexToMerge];
						Private::FBroadPhaseContext& OtherContext = BroadphaseContexts[BigIndexToMerge];

						TArray<FPBDCollisionConstraint*>& ThisArray = Context.CollisionContext.Allocator->NewActiveConstraints;
						TArray<FPBDCollisionConstraint*>& OtherArray = OtherContext.CollisionContext.Allocator->NewActiveConstraints;

						if (bIsDeterministic)
						{
							TArray<FPBDCollisionConstraint*> ResultArray;
							const int32 NumThis = ThisArray.Num();
							const int32 NumOther = OtherArray.Num();
							ResultArray.Reserve(NumThis + NumOther);

							int32 ThisIndex = 0;
							int32 OtherIndex = 0;

							while (ThisIndex < NumThis || OtherIndex < NumOther)
							{
								if (ThisIndex < NumThis && (OtherIndex >= NumOther ||
									ThisArray[ThisIndex]->GetContact().GetCollisionSortKey() < OtherArray[OtherIndex]->GetContact().GetCollisionSortKey()))
								{
									ResultArray.Add(ThisArray[ThisIndex++]);
								}
								else
								{
									check(OtherIndex < NumOther);
									ResultArray.Add(OtherArray[OtherIndex++]);
								}
							}
							ThisArray = MoveTemp(ResultArray);
						}
						else
						{
							ThisArray.Append(OtherArray);
						}
						OtherArray.Reset();
					};

				// This atomic is use to merge the sorted data in a safe multithreaded program. 
				// When reaching the end of a process, 
				// - either this atomic is INDEX_NONE, meaning there is no other batch to be merged with. So this atomic take the value of the batch
				// - either this atomic is the index to be merged with, so the merging is called with this current index, and the one stored in the atomic.
				std::atomic<int32> NextBatchToMerge = INDEX_NONE;
				if (bIsDeterministic)
				{
					TArray<UE::Tasks::FTask> ProcessSorting;
					for (int32 ContextIndex = 0; ContextIndex < PendingTasks.Num(); ContextIndex++)
					{
						int32 CurrentIndex = ContextsWithConstraints[ContextIndex];
						Private::FBroadPhaseContext& BroadphaseContext = BroadphaseContexts[CurrentIndex];
						UE::Tasks::FTask SortConstraintTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, &BroadphaseContext, CurrentIndex, &NextBatchToMerge, MergeLambda]()
							{
								QUICK_SCOPE_CYCLE_COUNTER(STAT_SortConstraintBatch);
								BroadphaseContext.CollisionContext.Allocator->NewActiveConstraints.Sort(
									[](const FPBDCollisionConstraintHandle& L, const FPBDCollisionConstraintHandle& R)
									{
										return L.GetContact().GetCollisionSortKey() < R.GetContact().GetCollisionSortKey();
									});
								int32 NextIndexToMerge = CurrentIndex;
								while (true)
								{
									int32 NextExpected = NextBatchToMerge.load(std::memory_order_relaxed);
									// If there is a sorted batch ready to be merged, lets merge it
									if (NextExpected != INDEX_NONE)
									{
										// Make sure the value hasn't been overwritten by another thread otherwise, retry
										if (NextBatchToMerge.compare_exchange_weak(NextExpected, INDEX_NONE, std::memory_order_relaxed))
										{
											int32 SmallIndexToMerge = FMath::Min(NextIndexToMerge, NextExpected);
											int32 BigIndexToMerge = FMath::Max(NextIndexToMerge, NextExpected);
											check(SmallIndexToMerge != BigIndexToMerge);
											MergeLambda(SmallIndexToMerge, BigIndexToMerge);
											NextIndexToMerge = SmallIndexToMerge;
										}
									}
									else // Otherwise if no batch are available set the current one as the next available for merging
									{
										int32 IndexNone = INDEX_NONE;
										// Make sure there is no available one, otherwise retry
										if (NextBatchToMerge.compare_exchange_weak(IndexNone, NextIndexToMerge, std::memory_order_relaxed))
										{
											break;
										}
									}
								}
							}, PendingTasks[ContextIndex]);
						ProcessSorting.Add(SortConstraintTask);
					}
					PendingTasks = MoveTemp(ProcessSorting);
				}
				else // For no deterministic program merge all context in new tasks, without sorting them
				{
					// Create a binary tree of task to merge all sorted constraints
					int32 NeighborStep = 1;
					while (PendingTasks.Num() >= 2)
					{
						int32 NumProcessSorting = PendingTasks.Num();
						TArray<UE::Tasks::FTask> ProcessMerging;
						// We merge data in the context that have a power of 2 index, so for first layer 0, 2, 4..., for the second layer 0, 4, 8... and so on.
						for (int32 ProcessIndex = 0, ContextIndex = 0; ProcessIndex < NumProcessSorting - 1; ProcessIndex += 2, ContextIndex += 2 * NeighborStep)
						{
							TArray<UE::Tasks::FTask> Preriq{ PendingTasks[ProcessIndex], PendingTasks[ProcessIndex + 1] };
							int32 SmallIndexToMerge = ContextsWithConstraints[ContextIndex];
							int32 BigIndexToMerge = ContextsWithConstraints[ContextIndex + NeighborStep];
							UE::Tasks::FTask MergeConstraintTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [SmallIndexToMerge, BigIndexToMerge, MergeLambda]()
								{
									QUICK_SCOPE_CYCLE_COUNTER(STAT_MergeConstraints);
									MergeLambda(SmallIndexToMerge, BigIndexToMerge);

								}, Preriq);
							ProcessMerging.Add(MergeConstraintTask);
						}
						// Add the last one for odd process number
						if (NumProcessSorting % 2 == 1)
						{
							ProcessMerging.Add(PendingTasks[NumProcessSorting - 1]);
						}
						PendingTasks = MoveTemp(ProcessMerging);
						NeighborStep *= 2;
					}
					ensure(PendingTasks.Num() <= 1);
				}
				UE::Tasks::Wait(PendingTasks);
			}

			// Lets move all constraints in the first allocator to be picked up by the CollisionConstraintAllocator
			if (!ContextsWithConstraints.IsEmpty() && ContextsWithConstraints[0] != 0)
			{
				BroadphaseContexts[0].CollisionContext.Allocator->NewActiveConstraints =
					MoveTemp(BroadphaseContexts[ContextsWithConstraints[0]].CollisionContext.Allocator->NewActiveConstraints);
			}

			// Update stats at the end in single thread when all mid phases have been computed.
			for (int32 ContextIndex = 0; ContextIndex < NumActiveBroadphaseContexts; ++ContextIndex)
			{
				NumBroadPhasePairs += BroadphaseContexts[ContextIndex].Overlaps.Num();
				NumMidPhases += BroadphaseContexts[ContextIndex].MidPhases.Num();
			}
		}

		/** @brief This function is the outer loop of collision detection. It loops over the
		 * particles view and do the broadphase + narrowphase collision detection
		 * @param OverlapView View to consider for the outer loop
		 * @param Dt Current simulation time step
		 * @param InSpatialAcceleration Spatial acceleration (AABB, bounding volumes...) to be used for broadphase collision detection
		 * @param NarrowPhase Narrowphase collision detection that will be executed on each potential pairs coming from the broadphase detection
		 * */
		template<bool bOnlyRigid, typename ViewType, typename SpatialAccelerationType>
		void ComputeParticlesOverlaps(
			ViewType& OverlapView, 
			FReal Dt,
			const SpatialAccelerationType& InSpatialAcceleration, 
			Private::FCollisionConstraintAllocator* Allocator,
			const FCollisionDetectorSettings& Settings)
		{
			// Reset all the contexts (we don't always use all of them, but don't reduce the array size so that the element arrays don't need reallocating)
			for (Private::FBroadPhaseContext& BroadphaseContext : BroadphaseContexts)
			{
				BroadphaseContext.Reset();
			}

			// The number of contexts that may have new overlaps in them
			NumActiveBroadphaseContexts = bSingleWorkerPhysics ? 1 : FMath::Max(FMath::Min(Chaos::CVars::NumWorkerCollisionFactor * FTaskGraphInterface::Get().GetNumWorkerThreads(), Chaos::MaxNumWorkers), 1);

			BroadphaseContexts.SetNum(NumActiveBroadphaseContexts);
			Allocator->SetMaxContexts(NumActiveBroadphaseContexts);

			for (int32 Index = 0; Index < NumActiveBroadphaseContexts; Index++)
			{
				Private::FCollisionContextAllocator* ContextAllocator = Allocator->GetContextAllocator(Index);
				BroadphaseContexts[Index].CollisionContext.SetSettings(Settings);
				BroadphaseContexts[Index].CollisionContext.SetAllocator(ContextAllocator);
			}

			using TSOA = typename ViewType::TSOA;
			using THandle = typename TSOA::THandleType;
			using THandleBase = typename THandle::THandleBase;
			PendingTasks.Reset();

			int32 ContextIndex = 0;

			for (int32 ViewIndex = 0; ViewIndex < OverlapView.SOAViews.Num(); ++ViewIndex)
			{
				const TSOAView<TSOA>& SOAView = OverlapView.SOAViews[ViewIndex];
				const int32 NumParticles = SOAView.Size();
				if (NumParticles == 0)
				{
					continue;
				}
				// Dispatch tasks according if it is an array of handles or a SOA
				if (const TArray<THandle*>* CurHandlesArray = SOAView.HandlesArray)
				{
					auto ReturnHandle = [CurHandlesArray](int32 Index) -> THandleBase
					{ 
						THandle* HandlePtr = (*CurHandlesArray)[Index];
						return THandleBase(HandlePtr->GeometryParticles, HandlePtr->ParticleIdx);
					};
					using LambdaType = decltype(ReturnHandle);
					const int32 NumHandles = CurHandlesArray->Num();
					if (bSingleWorkerPhysics)
					{
						ProduceParticleOverlapsSingleThreaded<bOnlyRigid, SpatialAccelerationType, THandle, LambdaType>(ContextIndex, NumHandles, InSpatialAcceleration, Dt, ReturnHandle);
					}
					else
					{
						DispatchTasks<bOnlyRigid, SpatialAccelerationType, THandle, LambdaType>(ContextIndex, NumHandles, InSpatialAcceleration, Dt, ReturnHandle);
					}
				}
				else if (TSOA* Soa = SOAView.SOA)
				{
					auto ReturnHandle = [Soa](int32 Index) { return THandleBase(Soa, Index); };
					using LambdaType = decltype(ReturnHandle);
					if (bSingleWorkerPhysics)
					{
						ProduceParticleOverlapsSingleThreaded<bOnlyRigid, SpatialAccelerationType, THandle, LambdaType>(ContextIndex, NumParticles, InSpatialAcceleration, Dt, ReturnHandle);
					}
					else
					{
						DispatchTasks<bOnlyRigid, SpatialAccelerationType, THandle, LambdaType>(ContextIndex, NumParticles, InSpatialAcceleration, Dt, ReturnHandle);
					}
				}
			}
		}

		template<typename T_SPATIALACCELERATION>
		void ProduceOverlaps(
			FReal Dt, 
			const T_SPATIALACCELERATION& InSpatialAcceleration, 
			Private::FCollisionConstraintAllocator* Allocator,
			const FCollisionDetectorSettings& Settings,
			IResimCacheBase* ResimCache
			)
		{
			// Select the set of particles that we loop over in the outer collision detection loop.
			// The goal is to detection all required collisions (dynamic-vs-everything) while not
			// visiting pairs that cannot collide (e.g., kinemtic-kinematic, or kinematic-sleeping for
			// stationary kinematics)
			const bool bDisableParallelFor = bDisableCollisionParallelFor;
			if(!bNeedsResim)
			{
				const TParticleView<TPBDRigidParticles<FReal, 3>>& DynamicSleepingView = Particles.GetNonDisabledDynamicView();
				const TParticleView<TPBDRigidParticles<FReal, 3>>& DynamicMovingKinematicView = Particles.GetActiveDynamicMovingKinematicParticlesView();

				// Usually we ignore sleeping particles in the outer loop and iterate over awake-dynamics and moving-kinematics. 
				// However, for scenes with a very large number of moving kinematics, it is faster to loop over awake-dynamics
				// and sleeping-dynamics, even though this means we visit sleeping pairs.
				if(DynamicSleepingView.Num() < DynamicMovingKinematicView.Num())
				{
					ComputeParticlesOverlaps<true>(DynamicSleepingView, Dt, InSpatialAcceleration, Allocator, Settings);
				}
				else
				{
					ComputeParticlesOverlaps<false>(DynamicMovingKinematicView, Dt, InSpatialAcceleration, Allocator, Settings);
				}
			}
			else
			{
				const TParticleView<TGeometryParticles<FReal, 3>>& DesyncedView = ResimCache->GetDesyncedView();
				
				ComputeParticlesOverlaps<false>(DesyncedView, Dt, InSpatialAcceleration, Allocator, Settings);
			}
		}

		FIgnoreCollisionManager& GetIgnoreCollisionManager() { return IgnoreCollisionManager; }

		// Stats
		int32 GetNumBroadPhasePairs() const
		{
			return NumBroadPhasePairs;
		}

		int32 GetNumMidPhases() const
		{
			return NumMidPhases;
		}

	private:
		template<bool bOnlyRigid, typename SpatialAccelerationType, typename THandle, typename LambdaType>
		void DispatchTasks(int32& ContextIndex, int32 NumParticles, const SpatialAccelerationType& InSpatialAcceleration, FReal Dt, LambdaType ReturnHandle)
		{
			using THandleBase = typename THandle::THandleBase;
			using TTransientHandle = typename THandle::TTransientHandle;

			const int32 ParticleByContext = FMath::Max(FMath::DivideAndRoundUp(NumParticles, NumActiveBroadphaseContexts), CollisionSmallBatchSize);
			const int32 NumBatch = FMath::DivideAndRoundUp(NumParticles, ParticleByContext);

			for (int32 BatchIndex = 0; BatchIndex < NumBatch; ContextIndex++, BatchIndex++)
			{
				if (ContextIndex >= NumActiveBroadphaseContexts)
				{
					ContextIndex = 0;
				}

				int32 EndIndex = (BatchIndex + 1) * ParticleByContext;
				EndIndex = BatchIndex == NumBatch - 1 ? FMath::Min(NumParticles, EndIndex) : EndIndex;

				if (ContextIndex >= PendingTasks.Num())
				{
					Private::FBroadPhaseContext& BroadPhaseContext = BroadphaseContexts[ContextIndex];
					const int32 StartIndex = BatchIndex * ParticleByContext;
					UE::Tasks::FTask PendingTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, &BroadPhaseContext, &InSpatialAcceleration, Dt, ReturnHandle, StartIndex, EndIndex]()
						{
							SCOPE_CYCLE_COUNTER(STAT_Collisions_ParticlePairBroadPhase);
							for (int32 Index = StartIndex; Index < EndIndex; Index++)
							{
								THandleBase Handle = ReturnHandle(Index);
								TTransientHandle& TransientHandle = static_cast<TTransientHandle&>(Handle);
								if (!TransientHandle.LightWeightDisabled())
								{
									ProduceParticleOverlaps<bOnlyRigid>(Dt, TransientHandle.Handle(), InSpatialAcceleration, BroadPhaseContext);
								}
							}
						}, UE::Tasks::ETaskPriority::High);
					PendingTasks.Add(PendingTask);
				}
				else
				{

					// If all context threads has been filled, create other tasks and link them to the previous one, 
					Private::FBroadPhaseContext& BroadPhaseContext = BroadphaseContexts[ContextIndex];
					const int32 StartIndex = BatchIndex * ParticleByContext;
					UE::Tasks::FTask PendingTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, &BroadPhaseContext, &InSpatialAcceleration, Dt, ReturnHandle, StartIndex, EndIndex]()
						{
							SCOPE_CYCLE_COUNTER(STAT_Collisions_ParticlePairBroadPhase);
							for (int32 Index = StartIndex; Index < EndIndex; Index++)
							{
								THandleBase Handle = ReturnHandle(Index);
								TTransientHandle& TransientHandle = static_cast<TTransientHandle&>(Handle);
								if (!TransientHandle.LightWeightDisabled())
								{
									ProduceParticleOverlaps<bOnlyRigid>(Dt, TransientHandle.Handle(), InSpatialAcceleration, BroadPhaseContext);
								}
							}
						}, PendingTasks[ContextIndex]);

					PendingTasks[ContextIndex] = PendingTask;
				}
			}
		}

		template<bool bOnlyRigid, typename SpatialAccelerationType, typename THandle, typename LambdaType>
		void ProduceParticleOverlapsSingleThreaded(int32& ContextIndex, int32 NumParticles, const SpatialAccelerationType& InSpatialAcceleration, FReal Dt, LambdaType ReturnHandle)
		{
			using THandleBase = typename THandle::THandleBase;
			using TTransientHandle = typename THandle::TTransientHandle;

			SCOPE_CYCLE_COUNTER(STAT_Collisions_ParticlePairBroadPhase);
			for (int32 Index = 0; Index < NumParticles; Index++)
			{
				THandleBase Handle = ReturnHandle(Index);
				TTransientHandle& TransientHandle = static_cast<TTransientHandle&>(Handle);
				if (!TransientHandle.LightWeightDisabled())
				{
					ProduceParticleOverlaps<bOnlyRigid>(Dt, TransientHandle.Handle(), InSpatialAcceleration, BroadphaseContexts[ContextIndex]);
				}
			}
		}

		// Generate the set of particles that overlap the specified particle and are allowed to collide with it
		template<bool bOnlyRigid, typename T_SPATIALACCELERATION>
		void ProduceParticleOverlaps(
			FReal Dt,
			FGeometryParticleHandle* Particle1,
			const T_SPATIALACCELERATION& InSpatialAcceleration,
			Private::FBroadPhaseContext& Context)
		{
			// @todo(chaos):We shouldn't need this data here (see uses below)
			bool bIsKinematic1 = true;
			bool bIsDynamicAwake1 = false;
			bool bIsDynamicAsleep1 = false;
			bool bDisabled1 = false;
			const FPBDRigidParticleHandle* Rigid1 = Particle1->CastToRigidParticle();
			if (Rigid1 != nullptr)
			{
				bIsKinematic1 = Rigid1->IsKinematic();
				bIsDynamicAsleep1 = !bIsKinematic1 && Rigid1->IsSleeping();
				bIsDynamicAwake1 = !bIsKinematic1 && !bIsDynamicAsleep1;
				bDisabled1 = Rigid1->Disabled();
			}

			// Skip particles we are not interested in
			// @todo(chaos) Ideally we would we should not be getting disabled particles, but we currently do from GeometryCollections.
			if (bDisabled1)
			{
				return;
			}

			// @todo(chaos): This should already be handled by selecting the appropriate particle view in the calling function. GeometryCollections?
			const bool bHasValidState = bOnlyRigid ? (bIsDynamicAwake1 || bIsDynamicAsleep1) : (bIsDynamicAwake1 || bIsKinematic1);
			if (!bHasValidState && !bNeedsResim)
			{
				return;
			}

			const bool bBody1Bounded = Particle1->HasBounds();
			if (bBody1Bounded)
			{
				const FAABB3 Box1 = Particle1->WorldSpaceInflatedBounds();

				{
					PHYSICS_CSV_SCOPED_EXPENSIVE(PhysicsVerbose, DetectCollisions_BroadPhase);
					Private::FSimOverlapVisitor OverlapVisitor(Particle1, Context);
					InSpatialAcceleration.Overlap(Box1, OverlapVisitor);
				}
			}
			else
			{
				const auto& GlobalElems = InSpatialAcceleration.GlobalObjects();
				Context.Overlaps.Reserve(GlobalElems.Num());

				for (auto& Elem : GlobalElems)
				{
					if (Particle1 != Elem.Payload.GetGeometryParticleHandle_PhysicsThread())
					{
						Context.Overlaps.Emplace(Particle1, Elem.Payload.GetGeometryParticleHandle_PhysicsThread(), 1);
					}
				}
			}
		}

		// Redistribute the midphases among the contexts to even out the per-core work a bit
		void RedistributeMidPhasesInContexts();

		// Find or assign midphases to each of the overlapping particle pairs
		// @todo(chaos): optimize
		void AssignMidPhases(Private::FBroadPhaseContext& BroadphaseContext);

		// Process all the midphases: generate constraints and execute the narrowphase
		void ProcessMidPhases(const FReal Dt, const Private::FBroadPhaseContext& BroadphaseContext);
		void CheckOverlapResults();

		const FPBDRigidsSOAs& Particles;
		const FAccelerationStructure* SpatialAcceleration;
		TArray<Private::FBroadPhaseContext> BroadphaseContexts;
		FIgnoreCollisionManager IgnoreCollisionManager;
		int32 NumActiveBroadphaseContexts;
		bool bNeedsResim;

		int32 NumBroadPhasePairs;
		int32 NumMidPhases;

		TArray<UE::Tasks::FTask> PendingTasks;
		TArray<UE::Tasks::FTask> BroadToNarrowTasks;
		// Contexts indexes in which there are constraints
		TArray<int32> ContextsWithConstraints;
	};
}
