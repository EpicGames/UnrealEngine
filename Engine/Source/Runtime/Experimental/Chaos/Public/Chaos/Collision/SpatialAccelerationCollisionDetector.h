// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Collision/CollisionDetector.h"
#include "Chaos/Collision/CollisionContext.h"
#include "Chaos/Collision/SpatialAccelerationBroadPhase.h"
#include "Chaos/EvolutionResimCache.h"
#include "Chaos/PBDCollisionConstraints.h"
#include "ProfilingDebugging/CsvProfiler.h"

namespace Chaos
{
	class FSpatialAccelerationCollisionDetector : public FCollisionDetector
	{
	public:
		FSpatialAccelerationCollisionDetector(FSpatialAccelerationBroadPhase& InBroadPhase, FPBDCollisionConstraints& InCollisionContainer)
			: FCollisionDetector(InCollisionContainer)
			, BroadPhase(InBroadPhase)
		{
		}

		FSpatialAccelerationBroadPhase& GetBroadPhase()
		{ 
			return BroadPhase;
		}

		void RunBroadPhase(const FReal Dt, FEvolutionResimCache* ResimCache)
		{
			if (!GetCollisionContainer().GetCollisionsEnabled())
			{
				return;
			}

			GetCollisionContainer().BeginDetectCollisions();

			// Run the broadphase and generate a midphase object for every overlapping particle pair
			BroadPhase.ProduceOverlaps(Dt, &GetCollisionContainer().GetConstraintAllocator(), GetCollisionContainer().GetDetectorSettings(), ResimCache);
		}

		void RunNarrowPhase(const FReal Dt, FEvolutionResimCache* ResimCache)
		{
			if (!GetCollisionContainer().GetCollisionsEnabled())
			{
				return;
			}

			// Run collision detection on the output of the broadphase
			{
				SCOPE_CYCLE_COUNTER(STAT_Collisions_NarrowPhase);
				CSV_SCOPED_TIMING_STAT(PhysicsVerbose, DetectCollisions_NarrowPhase);
				BroadPhase.ProduceCollisions(Dt);
				BroadPhase.GatherConstraints(GetCollisionContainer().GetIsDeterministic());
			}
			GetCollisionContainer().EndDetectCollisions();

			// If we have a resim cache restore and save contacts
			if (ResimCache)
			{
				// Ensure we have at least one allocator
				GetCollisionContainer().GetConstraintAllocator().SetMaxContexts(1);

				FCollisionContext Context;
				Context.SetSettings(GetCollisionContainer().GetDetectorSettings());
				Context.SetAllocator(GetCollisionContainer().GetConstraintAllocator().GetContextAllocator(0));

				GetCollisionContainer().GetConstraintAllocator().AddResimConstraints(ResimCache->GetAndSanitizeConstraints(), Context);

				ResimCache->SaveConstraints(GetCollisionContainer().GetConstraints());
			}
		}

		virtual void DetectCollisions(const FReal Dt, FEvolutionResimCache* ResimCache) override
		{
			SCOPE_CYCLE_COUNTER(STAT_Collisions_Detect);
			CHAOS_SCOPED_TIMER(DetectCollisions);
			CSV_SCOPED_TIMING_STAT(Chaos, DetectCollisions);

			RunBroadPhase(Dt, ResimCache);
			RunNarrowPhase(Dt, ResimCache);
		}

	private:
		FSpatialAccelerationBroadPhase& BroadPhase;
	};
}
