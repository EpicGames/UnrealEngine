// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Chaos/Core.h"
#include "Chaos/CollisionResolutionTypes.h"
#include "Chaos/ImplicitFwd.h"

namespace Chaos
{
	extern CHAOS_API void ConstructSphereTriangleOneShotManifold(const FImplicitSphere3& Sphere, const FTriangle& Triangle, const FReal CullDistance, FContactPointManifold& OutContactPoints);
}