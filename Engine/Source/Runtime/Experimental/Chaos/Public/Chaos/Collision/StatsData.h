// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/ChaosPerfTest.h"
#include "Chaos/Box.h"
#include "Chaos/ParticleHandle.h"

// Wrap use of FStatData in CHAOS_COLLISION_STAT to remove impact in shipping builds
#if UE_BUILD_SHIPPING
#define CHAOS_COLLISION_STAT(X)
#define CHAOS_COLLISION_STAT_DISABLED(X) X
#else
#define CHAOS_COLLISION_STAT(X) X;
#define CHAOS_COLLISION_STAT_DISABLED(X)
#endif

namespace Chaos
{
	namespace CollisionStats
	{

		static constexpr int32 BucketSizes2[] = { 0,1,4,8,16,32,64,128,512,MAX_int32 };

		template <bool bGatherStats>
		struct FStatHelper
		{
			int32 MaxCount;

			FStatHelper() {}
			void Record(int32 Count) {}
			FString ToString() const { return TEXT(""); }
		};

		template <>
		struct FStatHelper<true>
		{
			int32 BucketCount[UE_ARRAY_COUNT(BucketSizes2)];
			int32 MaxCount;

			FStatHelper()
			{
				FMemory::Memset(BucketCount, 0, sizeof(BucketCount));
				MaxCount = 0;
			}

			void Record(int32 Count)
			{
				for (int32 BucketIdx = 1; BucketIdx < UE_ARRAY_COUNT(BucketSizes2); ++BucketIdx)
				{
					if (Count >= BucketSizes2[BucketIdx - 1] && Count < BucketSizes2[BucketIdx])
					{
						++BucketCount[BucketIdx];
					}
				}

				if (Count > MaxCount)
				{
					MaxCount = Count;
				}
			}

			FString ToString() const
			{
				FString OutLog;
				int32 MaxBucketCount = 0;
				for (int32 Count : BucketCount)
				{
					if (Count > MaxBucketCount)
					{
						MaxBucketCount = Count;
					}
				}

				const float CountPerChar = static_cast<float>(MaxBucketCount) / 20.f;
				for (int32 Idx = 1; Idx < UE_ARRAY_COUNT(BucketSizes2); ++Idx)
				{
					int32 NumChars = static_cast<int32>(static_cast<float>(BucketCount[Idx]) / CountPerChar);
					if (Idx < UE_ARRAY_COUNT(BucketSizes2) - 1)
					{
						OutLog += FString::Printf(TEXT("\t[%4d - %4d) (%4d) |"), BucketSizes2[Idx - 1], BucketSizes2[Idx], BucketCount[Idx]);
					}
					else
					{
						OutLog += FString::Printf(TEXT("\t[%4d -  inf) (%4d) |"), BucketSizes2[Idx - 1], BucketCount[Idx]);
					}
					for (int32 Count = 0; Count < NumChars; ++Count)
					{
						OutLog += TEXT("-");
					}
					OutLog += TEXT("\n");
				}

				return OutLog;
			}
		};

		struct FStatDataImp
		{
			FStatDataImp() : CountNP(0), RejectedNP(0), SimulatedParticles(0), NumPotentials(0) {}


			void IncrementSimulatedParticles()
			{
				++SimulatedParticles;
			}

			void RecordBoundsData(const FAABB3& Box1)
			{
				BoundsDistribution.Record(static_cast<int32>(Box1.Extents().GetMax()));
			}

			void RecordBroadphasePotentials(int32 Num)
			{
				NumPotentials = Num;
				BroadphasePotentials.Record(Num);
			}

			void IncrementCountNP(int32 Count = 1)
			{
				CountNP += Count;
			}

			void IncrementRejectedNP()
			{
				++RejectedNP;
			}

			void FinalizeData()
			{
				NarrowPhasePerformed.Record(CountNP);
				const int32 NPSkipped = NumPotentials - CountNP;
				NarrowPhaseSkipped.Record(NPSkipped);
				NarrowPhaseRejected.Record(RejectedNP);
			}

			void Print()
			{
				FString OutLog;
#if CHAOS_PARTICLEHANDLE_TODO
				const float NumParticles = InParticles.Size();
				OutLog = FString::Printf(TEXT("ComputeConstraints stats:\n"
					"Total Particles:%d\nSimulated Particles:%d (%.2f%%)\n"
					"Max candidates per instance:%d (%.2f%%)\n"
					"Max candidates skipped per instance (NP skipped):%d (%.2f%%)\n"
					"Max narrow phase tests per instance:%d (%.2f%%)\n"
					"Max narrow phase rejected per instance (NP rejected):%d (%.2f%%)\n"
					"Constraints generated:%d\n"
				),
					InParticles.Size(),
					SimulatedParticles, SimulatedParticles / NumParticles * 100.f,
					BroadphasePotentials.MaxCount, BroadphasePotentials.MaxCount / NumParticles * 100.f,
					NarrowPhaseSkipped.MaxCount, NarrowPhaseSkipped.MaxCount / NumParticles * 100.f,
					NarrowPhasePerformed.MaxCount, NarrowPhasePerformed.MaxCount / NumParticles * 100.f,
					NarrowPhaseRejected.MaxCount, NarrowPhaseRejected.MaxCount / NumParticles * 100.f,
					Constraints.Num()
				);

				OutLog += FString::Printf(TEXT("Potentials per instance distribution:\n"));
				OutLog += BroadphasePotentials.ToString();


				OutLog += FString::Printf(TEXT("\nCandidates skipped per instance (NP skipped) distribution:\n"));
				OutLog += NarrowPhaseSkipped.ToString();

				OutLog += FString::Printf(TEXT("\nNarrow phase performed per instance distribution:\n"));
				OutLog += NarrowPhasePerformed.ToString();

				OutLog += FString::Printf(TEXT("\nNarrow phase candidates rejected per instance distribution:\n"));
				OutLog += NarrowPhaseRejected.ToString();

				OutLog += FString::Printf(TEXT("\nBounds distribution:\n"));
				OutLog += BoundsDistribution.ToString();

				UE_LOGF(LogChaos, Warning, "%ls", *OutLog);

				CHAOS_API extern bool bPendingHierarchyDump;
				bPendingHierarchyDump = false;
#endif
			}

			int32 CountNP;
			int32 RejectedNP;
			int32 SimulatedParticles;
			int32 NumPotentials;
			FStatHelper<true> BroadphasePotentials;
			FStatHelper<true> NarrowPhaseSkipped;
			FStatHelper<true> NarrowPhasePerformed;
			FStatHelper<true> NarrowPhaseRejected;
			FStatHelper<true> BoundsDistribution;
		};

		struct FStatData
		{
			FStatData(bool bGatherStats)
				: StatImp(nullptr)
			{
				CHAOS_COLLISION_STAT({ if (bGatherStats) { StatImp = new FStatDataImp(); } });
			}

			~FStatData()
			{
				CHAOS_COLLISION_STAT({ if (IsEnabled()) { delete StatImp; } });
			}

			FORCEINLINE bool IsEnabled() const
			{
				CHAOS_COLLISION_STAT({ return StatImp != nullptr; })
				CHAOS_COLLISION_STAT_DISABLED({ return false; })
			}

			FORCEINLINE void IncrementSimulatedParticles()
			{
				CHAOS_COLLISION_STAT({ if (IsEnabled()) { StatImp->IncrementSimulatedParticles(); } });
			}

			FORCEINLINE void RecordBoundsData(const FAABB3& Box1)
			{
				CHAOS_COLLISION_STAT({ if (IsEnabled()) { StatImp->RecordBoundsData(Box1); } });
			}

			FORCEINLINE void RecordBroadphasePotentials(int32 Num)
			{
				CHAOS_COLLISION_STAT({ if (IsEnabled()) { StatImp->RecordBroadphasePotentials(Num); } });
			}

			FORCEINLINE void IncrementCountNP(int32 Count = 1)
			{
				CHAOS_COLLISION_STAT({ if (IsEnabled()) { StatImp->IncrementCountNP(Count); } })
			}

			FORCEINLINE void IncrementRejectedNP()
			{
				CHAOS_COLLISION_STAT({ if (IsEnabled()) { StatImp->IncrementRejectedNP(); } });
			}

			FORCEINLINE void FinalizeData()
			{
				CHAOS_COLLISION_STAT({ if (IsEnabled()) { StatImp->FinalizeData(); } });
			}

			FORCEINLINE void Print()
			{
				CHAOS_COLLISION_STAT({ if (IsEnabled()) { StatImp->Print(); } });
			}

		private:
			FStatDataImp* StatImp;
		};


	}

}


