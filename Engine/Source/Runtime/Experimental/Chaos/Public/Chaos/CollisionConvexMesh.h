// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Chaos/Box.h"
#include "TriangleMesh.h"
#include "Particles.h"
#include "ChaosLog.h"
#include "Containers/ChunkedArray.h"
#include "CompGeom/ConvexHull3.h"

#define DEBUG_HULL_GENERATION 0

// Those flags allow to output geometric data in OBJ compatible format
// INSTRUCTIONS:
//     Get the data form the log, remove the log header part and save it to a .obj file,
//     in a 3D viewer or DCC ( Windows 3D Viewer, Maya, Blender ... ), open/import the .obj file
// WARNING: 
//    - this needs DEBUG_HULL_GENERATION to also be enabled
//    - this may produce a lot of data and slow down levels have assets with a lot of convexes
#define DEBUG_HULL_GENERATION_HULLSTEPS_TO_OBJ 0
#define DEBUG_HULL_GENERATION_BUILDHORIZON_TO_OBJ 0

namespace Chaos
{
	// When encountering a triangle or quad in hull generation (3-points or 4 coplanar points) we will instead generate
	// a prism with a small thickness to emulate the desired result as a hull. Otherwise hull generation will fail on
	// these cases. Verbose logging on LogChaos will point out when this path is taken for further scrutiny about
	// the geometry
	static constexpr FRealSingle TriQuadPrismInflation() { return 0.1f; }
	static constexpr FRealSingle DefaultHorizonEpsilon() { return 0.1f; }

	class FConvexBuilder
	{
	public:

		using FRealType = FRealSingle;
		using FVec3Type = TVec3<FRealType>;
		using FPlaneType = TPlaneConcrete<FRealType, 3>;
		using FAABB3Type = TAABB<FRealType, 3>;

		enum EBuildMethod
		{
			Default = 0,	// use what is set as default from cvars
			Original = 1,	// original method, fast but has issues
			ConvexHull3 = 2,// new method, slower but less issues
			ConvexHull3Simplified = 3, // same as above new method, plus new simplification method
		};

		class Params
		{
		public:
			Params()
				: HorizonEpsilon(DefaultHorizonEpsilon())
			{}

			FRealType HorizonEpsilon;
		};

		static FRealType SuggestEpsilon(const TArray<FVec3Type>& InVertices)
		{
			if (ComputeHorizonEpsilonFromMeshExtends == 0)
			{
				// legacy path, return the hardcoded default value
				return DefaultHorizonEpsilon();
			}

			// Create a scaled epsilon for our input data set. FLT_EPSILON is the distance between 1.0 and the next value
			// above 1.0 such that 1.0 + FLT_EPSILON != 1.0. Floats aren't equally disanced though so big or small numbers
			// don't work well with it. Here we take the max absolute of each axis and scale that for a wider margin and
			// use that to scale FLT_EPSILON to get a more relevant value.
			FVec3Type MaxAxes(TNumericLimits<FRealType>::Lowest());
			const int32 NumVertices = InVertices.Num();

			if (NumVertices <= 1)
			{
				return FLT_EPSILON;
			}

			for (int32 Index = 0; Index < NumVertices; ++Index)
			{
				FVec3Type PositionAbs = InVertices[Index].GetAbs();

				MaxAxes[0] = FMath::Max(MaxAxes[0], PositionAbs[0]);
				MaxAxes[1] = FMath::Max(MaxAxes[1], PositionAbs[1]);
				MaxAxes[2] = FMath::Max(MaxAxes[2], PositionAbs[2]);
			}

			return 3.0f * (MaxAxes[0] + MaxAxes[1] + MaxAxes[2]) * FLT_EPSILON;
		}

		static bool IsValidTriangle(const FVec3Type& A, const FVec3Type& B, const FVec3Type& C, FVec3Type& OutNormal)
		{
			const FVec3Type BA = B - A;
			const FVec3Type CA = C - A;
			const FVec3Type Cross = FVec3Type::CrossProduct(BA, CA);
			OutNormal = Cross.GetUnsafeNormal();
			return Cross.Size() > 1e-4;
		}

		static bool IsValidTriangle(const FVec3Type& A, const FVec3Type& B, const FVec3Type& C)
		{
			FVec3Type Normal(0);
			return IsValidTriangle(A, B, C, Normal);
		}

		static bool IsValidQuad(const FVec3Type& A, const FVec3Type& B, const FVec3Type& C, const FVec3Type& D, FVec3Type& OutNormal)
		{
			const UE::Math::TPlane<FRealType> TriPlane(A, B, C);
			const FRealType DPointDistance = FMath::Abs(TriPlane.PlaneDot(D));
			OutNormal = FVec3Type(TriPlane.X, TriPlane.Y, TriPlane.Z);
			return FMath::IsNearlyEqual(DPointDistance, FRealType(0), FRealType(UE_KINDA_SMALL_NUMBER));
		}
		
		static bool IsPlanarShape(const TArray<FVec3Type>& InVertices, FVec3Type& OutNormal)
		{
			bool bResult = false;
			const int32 NumVertices = InVertices.Num();
			
			if (NumVertices <= 3)
			{
				// Nothing, point, line or triangle, not a planar set
				return false;
			}
			else // > 3 points
			{
				const UE::Math::TPlane<FRealType> TriPlane(InVertices[0], InVertices[1], InVertices[2]);
				OutNormal = FVec3Type(TriPlane.X, TriPlane.Y, TriPlane.Z);

				for (int32 Index = 3; Index < NumVertices; ++Index)
				{
					const FRealType PointPlaneDot = FMath::Abs(TriPlane.PlaneDot(InVertices[Index]));
					if(!FMath::IsNearlyEqual(PointPlaneDot, FRealType(0), FRealType(UE_KINDA_SMALL_NUMBER)))
					{
						return false;
					}
				}
			}

			return true;
		}

	private:
		class FMemPool;
		struct FHalfEdge;
		struct FConvexFace
		{
			FHalfEdge* FirstEdge;
			FHalfEdge* ConflictList; //Note that these half edges are really just free verts grouped together
			FPlaneType Plane;
			FConvexFace* Prev;
			FConvexFace* Next; //these have no geometric meaning, just used for book keeping
			int32 PoolIdx;

		private:
			FConvexFace(int32 InPoolIdx, const FPlaneType& FacePlane)
				: PoolIdx(InPoolIdx)
			{
				Reset(FacePlane);
			}

			void Reset(const FPlaneType& FacePlane)
			{
				ConflictList = nullptr;
				Plane = FacePlane;
			}

			// Required for TChunkedArray
			FConvexFace() = default;
			~FConvexFace() = default;

			friend FMemPool;
			friend TChunkedArray<FConvexFace>;
		};

		struct FHalfEdge
		{
			int32 Vertex;
			FHalfEdge* Prev;
			FHalfEdge* Next;
			FHalfEdge* Twin;
			FConvexFace* Face;
			int32 PoolIdx;

		private:
			FHalfEdge(int32 InPoolIdx, int32 InVertex)
				: PoolIdx(InPoolIdx)
			{
				Reset(InVertex);
			}

			void Reset(int32 InVertex)
			{
				Vertex = InVertex;
			}

			// Required for TChunkedArray
			FHalfEdge() = default;
			~FHalfEdge() = default;

			friend FMemPool;
			friend TChunkedArray<FHalfEdge>;
		};

		class FMemPool
		{
		public:
			FHalfEdge* AllocHalfEdge(int32 InVertex =-1)
			{
				if(HalfEdgesFreeIndices.Num())
				{
					const uint32 Idx = HalfEdgesFreeIndices.Pop(EAllowShrinking::No);
					FHalfEdge* FreeHalfEdge = &HalfEdges[Idx];
					FreeHalfEdge->Reset(InVertex);
					ensure(FreeHalfEdge->PoolIdx == Idx);
					return FreeHalfEdge;
				}
				else
				{
					int32 Idx = HalfEdges.AddElement(FHalfEdge(HalfEdges.Num(), InVertex));
					return &HalfEdges[Idx];
				}
			}

			FConvexFace* AllocConvexFace(const FPlaneType& FacePlane)
			{
				if(FacesFreeIndices.Num())
				{
					const uint32 Idx = FacesFreeIndices.Pop(EAllowShrinking::No);
					FConvexFace* FreeFace = &Faces[Idx];
					FreeFace->Reset(FacePlane);
					ensure(FreeFace->PoolIdx == Idx);
					return FreeFace;
				}
				else
				{
					int32 Idx = Faces.AddElement(FConvexFace(Faces.Num(), FacePlane));
					return &Faces[Idx];
				}
			}

			void FreeHalfEdge(FHalfEdge* HalfEdge)
			{
				HalfEdgesFreeIndices.Add(HalfEdge->PoolIdx);
			}

			void FreeConvexFace(FConvexFace* Face)
			{
				FacesFreeIndices.Add(Face->PoolIdx);
			}

		private:
			TArray<int32> HalfEdgesFreeIndices;
			TChunkedArray<FHalfEdge> HalfEdges;

			TArray<int32> FacesFreeIndices;
			TChunkedArray<FConvexFace> Faces;
		};

	public:

		static CHAOS_API void Build(const TArray<FVec3Type>& InVertices, TArray<FPlaneType>& OutPlanes, TArray<TArray<int32>>& OutFaceIndices, TArray<FVec3Type>& OutVertices, FAABB3Type& OutLocalBounds, EBuildMethod BuildMethod = EBuildMethod::Default);

		static bool UseConvexHull3(EBuildMethod BuildMethod);

		static bool UseConvexHull3Simplifier(EBuildMethod BuildMethod);

		static CHAOS_API void BuildIndices(const TArray<FVec3Type>& InVertices, TArray<int32>& OutResultIndexData, EBuildMethod BuildMethod = EBuildMethod::Default);
		
		static void BuildConvexHull(const TArray<FVec3Type>& InVertices, TArray<TVec3<int32>>& OutIndices, const Params& InParams = Params())
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Chaos::BuildConvexHull);

			OutIndices.Reset();
			FMemPool Pool;
			FConvexFace* Faces = BuildInitialHull(Pool, InVertices);
			if(Faces == nullptr)
			{
				return;
			}

#if DEBUG_HULL_GENERATION
			FString InitialFacesString("Generated Initial Hull: ");
			FConvexFace* Current = Faces;
			while(Current)
			{
				InitialFacesString += FString::Printf(TEXT("(%d %d %d) "), Current->FirstEdge->Vertex, Current->FirstEdge->Next->Vertex, Current->FirstEdge->Prev->Vertex);
				Current = Current->Next;
			}
			UE_LOGF(LogChaos, VeryVerbose, "%ls", *InitialFacesString);
#endif

			FConvexFace* DummyFace = Pool.AllocConvexFace(Faces->Plane);
			DummyFace->Prev = nullptr;
			DummyFace->Next = Faces;
			Faces->Prev = DummyFace;

			FHalfEdge* ConflictV = FindConflictVertex(InVertices, DummyFace->Next);
			while(ConflictV)
			{

#if DEBUG_HULL_GENERATION
#if DEBUG_HULL_GENERATION_HULLSTEPS_TO_OBJ
				UE_LOGF(LogChaos, VeryVerbose, "# ======================================================");
				const FVec3Type ConflictPos = InVertices[ConflictV->Vertex];
				UE_LOGF(LogChaos, VeryVerbose, "# GENERATED HULL before adding Vtx %d (%f %f %f)", ConflictV->Vertex, ConflictPos.X, ConflictPos.Y, ConflictPos.Z);
				UE_LOGF(LogChaos, VeryVerbose, "# ------------------------------------------------------");
				FConvexFace* Face = DummyFace->Next;
				while (Face)
				{
					const FVector P1 = InVertices[Face->FirstEdge->Prev->Vertex];
					const FVector P2 = InVertices[Face->FirstEdge->Next->Vertex];
					const FVector P3 = InVertices[Face->FirstEdge->Vertex];
					UE_LOGF(LogChaos, VeryVerbose, "v %f %f %f", P1.X, P1.Y, P1.Z);
					UE_LOGF(LogChaos, VeryVerbose, "v %f %f %f", P2.X, P2.Y, P2.Z);
					UE_LOGF(LogChaos, VeryVerbose, "v %f %f %f", P3.X, P3.Y, P3.Z);
					UE_LOGF(LogChaos, VeryVerbose, "f -3 -2 -1");
					Face = Face->Next;
				}
#endif
#endif

				if (!AddVertex(Pool, InVertices, ConflictV, InParams))
				{
					// AddVertex failed to process the conflict vertex -- 
					// subsequent calls to FindConflictVertex will just keep finding the same one,
					// so all we can do here is fail to build a convex hull at all
					return;
				}
				ConflictV = FindConflictVertex(InVertices, DummyFace->Next);
			}

			FConvexFace* Cur = DummyFace->Next;
			while(Cur)
			{
				//todo(ocohen): this assumes faces are triangles, not true once face merging is added
				OutIndices.Add(TVec3<int32>(Cur->FirstEdge->Vertex, Cur->FirstEdge->Next->Vertex, Cur->FirstEdge->Next->Next->Vertex));
				FConvexFace* Next = Cur->Next;
				Cur = Next;
			}
		}

		static FTriangleMesh BuildConvexHullTriMesh(const TArray<FVec3Type>& InVertices)
		{
			TArray<TVec3<int32>> Indices;
			BuildConvexHull(InVertices, Indices);
			return FTriangleMesh(MoveTemp(Indices));
		}

		static bool IsPerformanceWarning(int32 NumPlanes, int32 NumVertices)
		{
			if (!PerformGeometryCheck)
			{
				return false;
			}

			return (NumVertices > VerticesThreshold);
		}

		static bool IsGeometryReductionEnabled()
		{
			return (PerformGeometryReduction>0)?true:false;
		}

		static FString PerformanceWarningString(int32 NumPlanes, int32 NumVertices)
		{
			return FString::Printf(TEXT("Planes %d, Vertices %d"), NumPlanes, NumVertices);
		}

		static void Simplify(TArray<FPlaneType>& InOutPlanes, TArray<TArray<int32>>& InOutFaces, TArray<FVec3Type>& InOutVertices, FAABB3Type& InOutLocalBounds)
		{
			struct TPair
			{
				TPair() : A(-1), B(-1) {}
				uint32 A;
				uint32 B;
			};

			uint32 NumberOfVerticesRequired = VerticesThreshold;
			uint32 NumberOfVerticesWeHave = InOutVertices.Num();
			int32 NumToDelete = NumberOfVerticesWeHave - NumberOfVerticesRequired;

			int32 Size = InOutVertices.Num();
			TArray<FVec3Type> Vertices(InOutVertices);

			TArray<bool> IsDeleted;
			IsDeleted.Reset();
			IsDeleted.Init(false, Size);

			if (NumToDelete > 0)
			{
				for (uint32 Iteration = 0; Iteration < (uint32)NumToDelete; Iteration++)
				{
					TPair ClosestPair;
					FRealType ClosestDistSqr = FLT_MAX;

					for (int32 A = 0; A < (Size - 1); A++)
					{
						if (!IsDeleted[A])
						{
							for (int32 B = A + 1; B < Size; B++)
							{
								if (!IsDeleted[B])
								{
									FVec3Type Vec = Vertices[A] - Vertices[B];
									FRealType LengthSqr = Vec.SizeSquared();
									if (LengthSqr < ClosestDistSqr)
									{
										ClosestDistSqr = LengthSqr;
										ClosestPair.A = A;
										ClosestPair.B = B;
									}
								}
							}
						}
					}

					if (ClosestPair.A != -1)
					{
						// merge to mid point
						Vertices[ClosestPair.A] = Vertices[ClosestPair.A] + (Vertices[ClosestPair.B] - Vertices[ClosestPair.A]) * 0.5f;
						IsDeleted[ClosestPair.B] = true;
					}
				}
			}

			TArray<FVec3Type> TmpVertices;
			for (int Idx = 0; Idx < Vertices.Num(); Idx++)
			{
				// Only add vertices that have not been merged away
				if (!IsDeleted[Idx])
				{
					TmpVertices.Add(Vertices[Idx]);
				}
			}

			Build(TmpVertices, InOutPlanes, InOutFaces, InOutVertices, InOutLocalBounds);
			check(InOutVertices.Num() > 3);
		}

		static bool IsFaceOutlineConvex(const FPlaneType& Plane, const TArray<int32>& Indices, const TArray<FVec3Type>& Vertices)
		{
			TArray<int8> Signs;
			Signs.SetNum(Indices.Num());

			if (Indices.Num() < 4)
			{
				return true;
			}

			for (int32 PointIndex = 0; PointIndex < Indices.Num(); PointIndex++)
			{
				const int32 Index0 = Indices[PointIndex];
				const int32 Index1 = Indices[(PointIndex + 1) % Indices.Num()];
				const int32 Index2 = Indices[(PointIndex + 2) % Indices.Num()];

				const FVec3Type Point0 = Vertices[Index0];
				const FVec3Type Point1 = Vertices[Index1];
				const FVec3Type Point2 = Vertices[Index2];

				const FVec3Type Segment0(Point1 - Point0);
				const FVec3Type Segment1(Point2 - Point1);

				const FVec3Type Cross = FVec3Type::CrossProduct(Segment0, Segment1);
				const FRealType Dot = FVec3Type::DotProduct(Cross, Plane.Normal());

				Signs[PointIndex] = static_cast<int8>(FMath::Sign(Dot));
			}

			int8 RefSign = 0;
			for (const int8 Sign : Signs)
			{
				if (RefSign == 0)
				{
					RefSign = Sign;
				}
				if (Sign != 0 && RefSign != Sign)
				{
					return false;
				}
			}

			return true;
		}

		// remove any invalid faces ( less than 3 vertices )
		static void RemoveInvalidFaces(TArray<FPlaneType>& InOutPlanes, TArray<TArray<int32>>& InOutFaceVertexIndices)
		{
			int32 PlaneIndex = 0;
			while (PlaneIndex < InOutPlanes.Num())
			{
				if (InOutFaceVertexIndices[PlaneIndex].Num() < 3)
				{
					InOutFaceVertexIndices.RemoveAtSwap(PlaneIndex);
					InOutPlanes.RemoveAtSwap(PlaneIndex);
				}
				else
				{
					++PlaneIndex;
				}
			}
		}
		
		// Convert multi-triangle faces to single n-gons
		static void MergeFaces(TArray<FPlaneType>& InOutPlanes, TArray<TArray<int32>>& InOutFaceVertexIndices, const TArray<FVec3Type>& Vertices, FRealType DistanceThreshold)
		{
			const FRealType NormalThreshold = 1.e-4f;

			// Find planes with equal normal within the threshold and merge them
			for (int32 PlaneIndex0 = 0; PlaneIndex0 < InOutPlanes.Num(); ++PlaneIndex0)
			{
				const FPlaneType& Plane0 = InOutPlanes[PlaneIndex0];
				TArray<int32>& Vertices0 = InOutFaceVertexIndices[PlaneIndex0];

				for (int32 PlaneIndex1 = PlaneIndex0 + 1; PlaneIndex1 < InOutPlanes.Num(); ++PlaneIndex1)
				{
					const FPlaneType& Plane1 = InOutPlanes[PlaneIndex1];
					const TArray<int32>& Vertices1 = InOutFaceVertexIndices[PlaneIndex1];

					// First similarity test: normals are close - this will reject all very dissimilar faces
					const FRealType PlaneNormalDot = FVec3Type::DotProduct(Plane0.Normal(), Plane1.Normal());
					if (PlaneNormalDot > 1.0f - NormalThreshold)
					{
						// Second similarity test: vertices of one plane are within threshold distance of the other. This is slower but more accurate
						bool bWithinDistanceThreshold = true;
						for (int32 Plane0VertexIndex : Vertices0)
						{
							const FVec3Type Plane0Vertex = Vertices[Plane0VertexIndex];
							const FRealType Plane0VertexDistance = FMath::Abs(FVec3Type::DotProduct(Plane1.X() - Plane0Vertex, Plane1.Normal()));
							if (Plane0VertexDistance > DistanceThreshold)
							{
								bWithinDistanceThreshold = false;
								break;
							}
						}
						if (bWithinDistanceThreshold)
						{
							for (int32 Plane1VertexIndex : Vertices1)
							{
								const FVec3Type Plane1Vertex = Vertices[Plane1VertexIndex];
								const FRealType Plane1VertexDistance = FMath::Abs(FVec3Type::DotProduct(Plane0.X() - Plane1Vertex, Plane0.Normal()));
								if (Plane1VertexDistance > DistanceThreshold)
								{
									bWithinDistanceThreshold = false;
									break;
								}
							}
						}

						if (bWithinDistanceThreshold)
						{
							// Merge the verts from the second plane into the first
							for (int32 VertexIndex1 = 0; VertexIndex1 < Vertices1.Num(); ++VertexIndex1)
							{
								Vertices0.AddUnique(Vertices1[VertexIndex1]);
							}

							// Erase the second plane
							InOutPlanes.RemoveAtSwap(PlaneIndex1, EAllowShrinking::No);
							InOutFaceVertexIndices.RemoveAtSwap(PlaneIndex1, EAllowShrinking::No);
							--PlaneIndex1;
						}
					}
				}
			}

			// Re-order the face vertices to form the face half-edges
			for (int32 PlaneIndex0 = 0; PlaneIndex0 < InOutPlanes.Num(); ++PlaneIndex0)
			{
				FinalizeFaces(InOutPlanes[PlaneIndex0], InOutFaceVertexIndices[PlaneIndex0], Vertices);
#if DEBUG_HULL_GENERATION
				ensure(IsFaceOutlineConvex(InOutPlanes[PlaneIndex0], InOutFaceVertexIndices[PlaneIndex0], Vertices));
#endif
			}
			
			RemoveInvalidFaces(InOutPlanes, InOutFaceVertexIndices);
		}

		// Find edge pairs that are colinear and remove the unnecessary vertex to make a single edge.
		// If we are left with an invalid face (2 verts or less), remove it.
		// NOTE: a vertex in the middle of two colinear edges on one face may still be required by some other face, 
		// although if we are lucky those faces would have been merged by the MergeFaces() function, depending on the tolerances used
		static void MergeColinearEdges(TArray<FPlaneType>& InOutPlanes, TArray<TArray<int32>>& InOutFaceVertexIndices, TArray<FVec3Type>& InOutVertices, FRealType AngleToleranceCos)
		{
			check(InOutPlanes.Num() == InOutFaceVertexIndices.Num());

			// This array maps from the input vertex to the output vertex array. INDEX_NONE means removed.
			TArray<int32> VertexIndexMap;
			VertexIndexMap.SetNum(InOutVertices.Num());
			for (int32& VertexIndex : VertexIndexMap)
			{
				VertexIndex = INDEX_NONE;
			}

			// See if we have any co-linear edges in any faces, where the center vertex
			// is not required by some other face. Assume we already removed coincident vertices.
			// NOTE: after this loop the vertex index map contains INDEX_NONE for items to remove
			// and other vertex has its original index. We pack and re-index later.
			for (int32 PlaneIndex0 = 0; PlaneIndex0 < InOutFaceVertexIndices.Num(); ++PlaneIndex0)
			{
				TArray<int32>& FaceVertexIndices = InOutFaceVertexIndices[PlaneIndex0];
				if (FaceVertexIndices.Num() > 2)
				{
					// Visit all set of 3-vertex chains in the face
					int32 VertexIndex0 = FaceVertexIndices[FaceVertexIndices.Num() - 2];
					int32 VertexIndex1 = FaceVertexIndices[FaceVertexIndices.Num() - 1];
					for (int32 FaceVertexIndex = 0; FaceVertexIndex < FaceVertexIndices.Num(); ++FaceVertexIndex)
					{
						int32 VertexIndex2 = FaceVertexIndices[FaceVertexIndex];

						// Calculate the sine of the angle between the two edges formed by the 3 vertices
						const FVec3 Edge0 = (InOutVertices[VertexIndex1] - InOutVertices[VertexIndex0]).GetSafeNormal();
						const FVec3 Edge1 = (InOutVertices[VertexIndex2] - InOutVertices[VertexIndex1]).GetSafeNormal();
						const FReal CosAngle = FVec3::DotProduct(Edge0, Edge1);

						// See if we need the vertex.
						if (CosAngle < (FReal(1) - AngleToleranceCos))
						{
							VertexIndexMap[VertexIndex1] = VertexIndex1;
						}

						// Move to next edge pair
						VertexIndex0 = VertexIndex1;
						VertexIndex1 = VertexIndex2;
					}
				}
			}

			// Remove unused vertices from all faces and update the index map to account for removals
			int32 NumVerticesRemoved = 0;
			for (int32 VertexIndex = 0; VertexIndex < InOutVertices.Num(); ++VertexIndex)
			{
				if (VertexIndexMap[VertexIndex] == INDEX_NONE)
				{
					// Remove vertices we don't need from faces that use them
					// If we end up with less than 3 verts, remove the face
					for (int32 PlaneIndex0 = InOutFaceVertexIndices.Num() - 1; PlaneIndex0 >= 0; --PlaneIndex0)
					{
						InOutFaceVertexIndices[PlaneIndex0].Remove(VertexIndex);
						if (InOutFaceVertexIndices[PlaneIndex0].Num() < 3)
						{
							InOutFaceVertexIndices.RemoveAt(PlaneIndex0);
							InOutPlanes.RemoveAt(PlaneIndex0);
						}
					}
					++NumVerticesRemoved;
				}
				else
				{
					VertexIndexMap[VertexIndex] = VertexIndex - NumVerticesRemoved;
				}
			}

			if (NumVerticesRemoved > 0)
			{
				// Remove unused verts from the array
				for (int32 VertexIndex = InOutVertices.Num() - 1; VertexIndex >= 0; --VertexIndex)
				{
					if (VertexIndexMap[VertexIndex] == INDEX_NONE)
					{
						InOutVertices.RemoveAt(VertexIndex);
					}
				}

				// Remap vertex indices in all faces
				for (int32 PlaneIndex0 = 0; PlaneIndex0 < InOutFaceVertexIndices.Num(); ++PlaneIndex0)
				{
					TArray<int32>& FaceVertexIndices = InOutFaceVertexIndices[PlaneIndex0];
					for (int32 FaceVertexIndex = FaceVertexIndices.Num() - 1; FaceVertexIndex >= 0; --FaceVertexIndex)
					{
						FaceVertexIndices[FaceVertexIndex] = VertexIndexMap[FaceVertexIndices[FaceVertexIndex]];
					}
				}
			}
		}

		// IMPORTANT : vertices are assumed to be sorted CCW
		static void RemoveInsideFaceVertices(const FPlaneType& Face, TArray<int32>& InOutFaceVertexIndices, const TArray<FVec3Type>& Vertices, const FVec3Type& Centroid)
		{
			// we let 3 points faces being processed as there may be colinear cases that will result n invalid faces out of this function?
			if (InOutFaceVertexIndices.Num() < 3)
			{
			 	return;
			}
			// find furthest point from centroid as it is garanteed to be part of the convex hull 
			int32 StartIndex = 0;
			FRealType FurthestSquaredDistance = TNumericLimits<FRealType>::Lowest();
			for (int32 Index = 0; Index < InOutFaceVertexIndices.Num(); ++Index)
			{
				const FRealType SquaredDistance = (Vertices[InOutFaceVertexIndices[Index]] - Centroid).SquaredLength();
				if (SquaredDistance > FurthestSquaredDistance)
				{
					StartIndex = Index;
					FurthestSquaredDistance = SquaredDistance;
				}
			}

			const int32 VtxCount = InOutFaceVertexIndices.Num();

			struct FPointSegment
			{
				int32 VtxIndex;
				FVec3Type Segment;
			};

			const int32 VtxStartIndex = InOutFaceVertexIndices[StartIndex];
			int32 VtxIndex0 = VtxStartIndex;
			int32 VtxIndex1 = InOutFaceVertexIndices[(StartIndex + 1) % VtxCount];

			TArray<FPointSegment> Stack;
			Stack.Push({ VtxIndex0, FVec3Type{0} });
			Stack.Push({ VtxIndex1, Vertices[VtxIndex1] - Vertices[VtxIndex0] });

			int32 Step = 2; // we already processed the two first 
			while (Step <= VtxCount)
			{
				const int32 NextIndex = (StartIndex + Step) % VtxCount;

				const FPointSegment& LastOnStack = Stack.Last();
				VtxIndex0 = LastOnStack.VtxIndex;
				VtxIndex1 = InOutFaceVertexIndices[NextIndex];
				const FVec3Type Segment = Vertices[VtxIndex1] - Vertices[VtxIndex0];

				const FVec3 Cross = FVec3Type::CrossProduct(LastOnStack.Segment, Segment);
				if (FVec3Type::DotProduct(Cross, Face.Normal()) >= 0)
				{
					if (VtxIndex1 != VtxStartIndex)
					{
						Stack.Push({ VtxIndex1, Segment });
					}
					Step++;
				}
				else
				{
					Stack.Pop();
				}
			}

			// faces where all points are on the same line may produce 2 points faces after removal
			// in that case let's keep the entire face empty as a sign this should be trimmed 
			InOutFaceVertexIndices.Reset();
			if (Stack.Num() >= 3)
			{
				for (const FPointSegment& PointSegment : Stack)
				{
					InOutFaceVertexIndices.Add(PointSegment.VtxIndex);
				}
			}
		}

		// Reorder the vertices to be counter-clockwise about the normal
		static void SortFaceVerticesCCW(const FPlaneType& Face, TArray<int32>& InOutFaceVertexIndices, const TArray<FVec3Type>& Vertices, const FVec3Type& Centroid)
		{
			const FMatrix33 FaceMatrix = (FMatrix44f)FRotationMatrix::MakeFromZ(FVector(Face.Normal()));

			// [2, -2] based on clockwise angle about the normal
			auto VertexScore = [&Centroid, &FaceMatrix, &Vertices](int32 VertexIndex)
			{
				const FVec3Type CentroidOffsetDir = (Vertices[VertexIndex] - Centroid).GetSafeNormal();
				const FRealType DotX = FVec3Type::DotProduct(CentroidOffsetDir, FaceMatrix.GetAxis(0));
				const FRealType DotY = FVec3Type::DotProduct(CentroidOffsetDir, FaceMatrix.GetAxis(1));
				const FRealType Score = (DotX >= 0.0f) ? 1.0f + DotY : -1.0f - DotY;
				return Score;
			};

			auto VertexSortPredicate = [&VertexScore](int32 LIndex, int32 RIndex)
			{
				return VertexScore(LIndex) < VertexScore(RIndex);
			};

			InOutFaceVertexIndices.Sort(VertexSortPredicate);
		}

		// make sure faces have CCW winding, remove inside points 
		static void FinalizeFaces(const FPlaneType& Face, TArray<int32>& InOutFaceVertexIndices, const TArray<FVec3Type>& Vertices)
		{
			// compute centroid as this is needed for both sorting and removing inside points
			FVec3Type Centroid(0);
			for (int32 VertexIndex = 0; VertexIndex < InOutFaceVertexIndices.Num(); ++VertexIndex)
			{
				Centroid += Vertices[InOutFaceVertexIndices[VertexIndex]];
			}
			Centroid /= FRealType(InOutFaceVertexIndices.Num());

			SortFaceVerticesCCW(Face, InOutFaceVertexIndices, Vertices, Centroid);
			
			RemoveInsideFaceVertices(Face, InOutFaceVertexIndices, Vertices, Centroid);
		}

		// Generate the vertex indices for all planes in CCW order (used to serialize old data that did not have structure data)
		static void BuildPlaneVertexIndices(TArray<FPlaneType>& InPlanes, const TArray<FVec3Type>& Vertices, TArray<TArray<int32>>& OutFaceVertexIndices, const FRealType DistanceTolerance = 1.e-3f)
		{
			OutFaceVertexIndices.Reset(InPlanes.Num());
			for (int32 PlaneIndex = 0; PlaneIndex < InPlanes.Num(); ++PlaneIndex)
			{
				for (int32 VertexIndex = 0; VertexIndex < Vertices.Num(); ++VertexIndex)
				{
					const FRealType PlaneVertexDistance = FVec3Type::DotProduct(InPlanes[PlaneIndex].Normal(), Vertices[VertexIndex] - InPlanes[PlaneIndex].X());
					if (FMath::Abs(PlaneVertexDistance) < DistanceTolerance)
					{
						OutFaceVertexIndices[PlaneIndex].Add(VertexIndex);
					}
				}

				FinalizeFaces(InPlanes[PlaneIndex], OutFaceVertexIndices[PlaneIndex], Vertices);
			}
			RemoveInvalidFaces(InPlanes, OutFaceVertexIndices);
		}

		// CVars variables for controlling geometry complexity checking and simplification
		static CHAOS_API int32 PerformGeometryCheck;
		static CHAOS_API int32 PerformGeometryReduction;
		static CHAOS_API int32 VerticesThreshold;
		static CHAOS_API int32 ComputeHorizonEpsilonFromMeshExtends;
		static CHAOS_API bool bUseGeometryTConvexHull3;
		static CHAOS_API bool bUseSimplifierForTConvexHull3;

	private:

		static FVec3Type ComputeFaceNormal(const FVec3Type& A, const FVec3Type& B, const FVec3Type& C)
		{
			return FVec3Type::CrossProduct((B - A), (C - A));
		}

		static FConvexFace* CreateFace(FMemPool& Pool, const TArray<FVec3Type>& InVertices, FHalfEdge* RS, FHalfEdge* ST, FHalfEdge* TR)
		{
			RS->Prev = TR;
			RS->Next = ST;
			ST->Prev = RS;
			ST->Next = TR;
			TR->Prev = ST;
			TR->Next = RS;
			FVec3Type RSTNormal = ComputeFaceNormal(InVertices[RS->Vertex], InVertices[ST->Vertex], InVertices[TR->Vertex]);
			const FRealType RSTNormalSize = RSTNormal.Size();
			if (RSTNormalSize > 0)
			{
				RSTNormal = RSTNormal * (1 / RSTNormalSize);
			}
			else
			{
				// degenerated face, use a valid neighbor face normal 
				RSTNormal = RS->Twin->Face->Plane.Normal();
			}
			FConvexFace* RST = Pool.AllocConvexFace(FPlaneType(InVertices[RS->Vertex], RSTNormal));
			RST->FirstEdge = RS;
			RS->Face = RST;
			ST->Face = RST;
			TR->Face = RST;
			return RST;
		}

		static void StealConflictList(FMemPool& Pool, const TArray<FVec3Type>& InVertices, FHalfEdge* OldList, FConvexFace** Faces, int32 NumFaces)
		{
			FHalfEdge* Cur = OldList;
			while(Cur)
			{
				FRealType MaxD = 1e-4f;
				int32 MaxIdx = -1;
				for(int32 Idx = 0; Idx < NumFaces; ++Idx)
				{
					FRealType Distance = Faces[Idx]->Plane.SignedDistance(InVertices[Cur->Vertex]);
					if(Distance > MaxD)
					{
						MaxD = Distance;
						MaxIdx = Idx;
					}
				}

				bool bDeleteVertex = MaxIdx == -1;
				if(!bDeleteVertex)
				{
					//let's make sure faces created with this new conflict vertex will be valid. The plane check above is not sufficient because long thin triangles will have a plane with its point at one of these. Combined with normal and precision we can have errors
					auto PretendNormal = [&InVertices](FHalfEdge* A, FHalfEdge* B, FHalfEdge* C) {
						return FVec3Type::CrossProduct(InVertices[B->Vertex] - InVertices[A->Vertex], InVertices[C->Vertex] - InVertices[A->Vertex]).SizeSquared();
					};
					FHalfEdge* Edge = Faces[MaxIdx]->FirstEdge;
					do
					{
						if(PretendNormal(Edge->Prev, Edge, Cur) < 1e-4)
						{
							bDeleteVertex = true;
							break;
						}
						Edge = Edge->Next;
					} while(Edge != Faces[MaxIdx]->FirstEdge);
				}

				if(!bDeleteVertex)
				{
					FHalfEdge* Next = Cur->Next;
					FHalfEdge*& ConflictList = Faces[MaxIdx]->ConflictList;
					if(ConflictList)
					{
						ConflictList->Prev = Cur;
					}
					Cur->Next = ConflictList;
					ConflictList = Cur;
					Cur->Prev = nullptr;
					Cur = Next;
				}
				else
				{
					//point is contained, we can delete it
					FHalfEdge* Next = Cur->Next;
					Pool.FreeHalfEdge(Cur);
					Cur = Next;
				}
			}
		}

		static FConvexFace* BuildInitialHull(FMemPool& Pool, const TArray<FVec3Type>& InVertices)
		{
			if (InVertices.Num() < 4) //not enough points
			{
				return nullptr;
			}

			constexpr FRealType Epsilon = 1e-4f;

			const int32 NumVertices = InVertices.Num();

			//We store the vertex directly in the half-edge. We use its next to group free vertices by context list
			//create a starting triangle by finding min/max on X and max on Y
			FRealType MinX = TNumericLimits<FRealType>::Max();
			FRealType MaxX = TNumericLimits<FRealType>::Lowest();
			FHalfEdge* A = nullptr; //min x
			FHalfEdge* B = nullptr; //max x
			FHalfEdge* DummyHalfEdge = Pool.AllocHalfEdge(-1);
			DummyHalfEdge->Prev = nullptr;
			DummyHalfEdge->Next = nullptr;
			FHalfEdge* Prev = DummyHalfEdge;

			for (int32 i = 0; i < NumVertices; ++i)
			{
				FHalfEdge* VHalf = Pool.AllocHalfEdge(i); //todo(ocohen): preallocate these
				Prev->Next = VHalf;
				VHalf->Prev = Prev;
				VHalf->Next = nullptr;
				const FVec3Type& V = InVertices[i];

				if(V[0] < MinX)
				{
					MinX = V[0];
					A = VHalf;
				}
				if(V[0] > MaxX)
				{
					MaxX = V[0];
					B = VHalf;
				}

				Prev = VHalf;
			}

			check(A && B);
			if (A == B || (InVertices[A->Vertex] - InVertices[B->Vertex]).SizeSquared() < Epsilon) //infinitely thin
			{
				return nullptr;
			}

			//remove A and B from conflict list
			A->Prev->Next = A->Next;
			if(A->Next)
			{
				A->Next->Prev = A->Prev;
			}
			B->Prev->Next = B->Next;
			if(B->Next)
			{
				B->Next->Prev = B->Prev;
			}

			//find C so that we get the biggest base triangle
			FRealType MaxTriSize = Epsilon;
			const FVec3Type AToB = InVertices[B->Vertex] - InVertices[A->Vertex];
			FHalfEdge* C = nullptr;
			for(FHalfEdge* V = DummyHalfEdge->Next; V; V = V->Next)
			{
				FRealType TriSize = FVec3Type::CrossProduct(AToB, InVertices[V->Vertex] - InVertices[A->Vertex]).SizeSquared();
				if(TriSize > MaxTriSize)
				{
					MaxTriSize = TriSize;
					C = V;
				}
			}

			if(C == nullptr) //biggest triangle is tiny
			{
				return nullptr;
			}

			//remove C from conflict list
			C->Prev->Next = C->Next;
			if(C->Next)
			{
				C->Next->Prev = C->Prev;
			}

			//find farthest D along normal
			const FVec3Type AToC = InVertices[C->Vertex] - InVertices[A->Vertex];
			const FVec3Type Normal = FVec3Type::CrossProduct(AToB, AToC).GetSafeNormal();

			FRealType MaxPosDistance = Epsilon;
			FRealType MaxNegDistance = Epsilon;
			FHalfEdge* PosD = nullptr;
			FHalfEdge* NegD = nullptr;
			for(FHalfEdge* V = DummyHalfEdge->Next; V; V = V->Next)
			{
				FRealType Dot = FVec3Type::DotProduct(InVertices[V->Vertex] - InVertices[A->Vertex], Normal);
				if(Dot > MaxPosDistance)
				{
					MaxPosDistance = Dot;
					PosD = V;
				}
				if(-Dot > MaxNegDistance)
				{
					MaxNegDistance = -Dot;
					NegD = V;
				}
			}

			if(MaxNegDistance == Epsilon && MaxPosDistance == Epsilon)
			{
				return nullptr; //plane
			}

			const bool bPositive = MaxNegDistance < MaxPosDistance;
			FHalfEdge* D = bPositive ? PosD : NegD;
			check(D);

			//remove D from conflict list
			D->Prev->Next = D->Next;
			if(D->Next)
			{
				D->Next->Prev = D->Prev;
			}

			//we must now create the 3 faces. Face must be oriented CCW around normal and positive normal should face out
			//Note we are now using A,B,C,D as edges. We can only use one edge per face so once they're used we'll need new ones
			FHalfEdge* Edges[4] = {A, B, C, D};

			//The base is a plane with Edges[0], Edges[1], Edges[2]. The order depends on which side D is on
			if(bPositive)
			{
				//D is on the positive side of Edges[0], Edges[1], Edges[2] so we must reorder it
				FHalfEdge* Tmp = Edges[0];
				Edges[0] = Edges[1];
				Edges[1] = Tmp;
			}

			FConvexFace* Faces[4];
			Faces[0] = CreateFace(Pool, InVertices, Edges[0], Edges[1], Edges[2]); //base
			Faces[1] = CreateFace(Pool, InVertices, Pool.AllocHalfEdge(Edges[1]->Vertex), Pool.AllocHalfEdge(Edges[0]->Vertex), Edges[3]);
			Faces[2] = CreateFace(Pool, InVertices, Pool.AllocHalfEdge(Edges[0]->Vertex), Pool.AllocHalfEdge(Edges[2]->Vertex), Pool.AllocHalfEdge(Edges[3]->Vertex));
			Faces[3] = CreateFace(Pool, InVertices, Pool.AllocHalfEdge(Edges[2]->Vertex), Pool.AllocHalfEdge(Edges[1]->Vertex), Pool.AllocHalfEdge(Edges[3]->Vertex));

			auto MakeTwins = [](FHalfEdge* E1, FHalfEdge* E2) {
				E1->Twin = E2;
				E2->Twin = E1;
			};
			//mark twins so half edge can cross faces
			MakeTwins(Edges[0], Faces[1]->FirstEdge); //0-1 1-0
			MakeTwins(Edges[1], Faces[3]->FirstEdge); //1-2 2-1
			MakeTwins(Edges[2], Faces[2]->FirstEdge); //2-0 0-2
			MakeTwins(Faces[1]->FirstEdge->Next, Faces[2]->FirstEdge->Prev); //0-3 3-0
			MakeTwins(Faces[1]->FirstEdge->Prev, Faces[3]->FirstEdge->Next); //3-1 1-3
			MakeTwins(Faces[2]->FirstEdge->Next, Faces[3]->FirstEdge->Prev); //2-3 3-2

			Faces[0]->Prev = nullptr;
			for(int i = 1; i < 4; ++i)
			{
				Faces[i - 1]->Next = Faces[i];
				Faces[i]->Prev = Faces[i - 1];
			}
			Faces[3]->Next = nullptr;

			//split up the conflict list
			StealConflictList(Pool, InVertices, DummyHalfEdge->Next, Faces, 4);
			return Faces[0];
		}

		static FHalfEdge* FindConflictVertex(const TArray<FVec3Type>& InVertices, FConvexFace* FaceList)
		{
			UE_CLOGF(DEBUG_HULL_GENERATION, LogChaos, VeryVerbose, "Finding conflict vertex");

			for(FConvexFace* CurFace = FaceList; CurFace; CurFace = CurFace->Next)
			{
				UE_CLOGF(DEBUG_HULL_GENERATION, LogChaos, VeryVerbose, "\tTesting Face (%d %d %d)", CurFace->FirstEdge->Vertex, CurFace->FirstEdge->Next->Vertex, CurFace->FirstEdge->Prev->Vertex);

				FRealType MaxD = TNumericLimits<FRealType>::Lowest();
				FHalfEdge* MaxV = nullptr;
				for(FHalfEdge* CurFaceVertex = CurFace->ConflictList; CurFaceVertex; CurFaceVertex = CurFaceVertex->Next)
				{
					//is it faster to cache this from stealing stage?
					FRealType Dist = FVec3Type::DotProduct(InVertices[CurFaceVertex->Vertex], CurFace->Plane.Normal());
					if(Dist > MaxD)
					{
						MaxD = Dist;
						MaxV = CurFaceVertex;
					}
				}

				UE_CLOGF((DEBUG_HULL_GENERATION && !MaxV), LogChaos, VeryVerbose, "\t\tNo Conflict List");
				UE_CLOGF((DEBUG_HULL_GENERATION && MaxV), LogChaos, VeryVerbose, "\t\tFound %d at distance %f", MaxV->Vertex, MaxD);

				check(CurFace->ConflictList == nullptr || MaxV);
				if(MaxV)
				{
					if(MaxV->Prev)
					{
						MaxV->Prev->Next = MaxV->Next;
					}
					if(MaxV->Next)
					{
						MaxV->Next->Prev = MaxV->Prev;
					}
					if(MaxV == CurFace->ConflictList)
					{
						CurFace->ConflictList = MaxV->Next;
					}
					MaxV->Face = CurFace;
					return MaxV;
				}
			}

			return nullptr;
		}

		static void BuildHorizon(const TArray<FVec3Type>& InVertices, FHalfEdge* ConflictV, TArray<FHalfEdge*>& HorizonEdges, TArray<FConvexFace*>& FacesToDelete, const Params& InParams)
		{
#if DEBUG_HULL_GENERATION
			UE_LOGF(LogChaos, VeryVerbose, "Generate horizon - START");
#endif
			//We must flood fill from the initial face and mark edges of faces the conflict vertex cannot see
			//In order to return a CCW ordering we must traverse each face in CCW order from the edge we crossed over
			//This should already be the ordering in the half edge
			const FRealType Epsilon = InParams.HorizonEpsilon;
			const FVec3Type V = InVertices[ConflictV->Vertex];
			TSet<FConvexFace*> Processed;
			TArray<FHalfEdge*> Queue;
			check(ConflictV->Face);
			Queue.Add(ConflictV->Face->FirstEdge->Prev); //stack pops so reverse order
			Queue.Add(ConflictV->Face->FirstEdge->Next);
			Queue.Add(ConflictV->Face->FirstEdge);
			FacesToDelete.Add(ConflictV->Face);
			while(Queue.Num())
			{
#if DEBUG_HULL_GENERATION
				FString QueueString("\t Queue (");
				for (const FHalfEdge* QueuedEdge : Queue)
				{
					QueueString += FString::Printf(TEXT(" [%d - %d] "), QueuedEdge->Vertex, QueuedEdge->Next->Vertex);
				}
				QueueString += TEXT(")");
				UE_LOGF(LogChaos, VeryVerbose, "%ls", *QueueString);
#endif 

				FHalfEdge* Edge = Queue.Pop(EAllowShrinking::No);
				Processed.Add(Edge->Face);
				FHalfEdge* Twin = Edge->Twin;
				FConvexFace* NextFace = Twin->Face;

#if DEBUG_HULL_GENERATION
				UE_LOGF(LogChaos, VeryVerbose, "\tPop edge [%d - %d] from queue", Edge->Vertex, Edge->Next->Vertex);
#endif 

				if(Processed.Contains(NextFace))
				{
#if DEBUG_HULL_GENERATION
					UE_LOGF(LogChaos, VeryVerbose, "\tTwin Face [%d] already processed - skip", NextFace);
#endif
					continue;
				}
				const FRealType Distance = NextFace->Plane.SignedDistance(V);
				if(Distance > Epsilon)
				{
#if DEBUG_HULL_GENERATION
					UE_LOGF(LogChaos, VeryVerbose, "\tDistance [%f] > Epsilon [%f] - add to queue", Distance, Epsilon);
#endif 
					Queue.Add(Twin->Prev); //stack pops so reverse order
					Queue.Add(Twin->Next);
					FacesToDelete.Add(NextFace);
				}
				else
				{
#if DEBUG_HULL_GENERATION
					UE_LOGF(LogChaos, VeryVerbose, "\tAdd [%d - %d] to horizon ", Edge->Vertex, Edge->Next->Vertex);
#endif 
					// we need to ensure that the horizon is a continous edge loop
					// this may get the wrong edge path, but way more roibust than not testing this
					if (HorizonEdges.Num() == 0 || Edge->Vertex == HorizonEdges[HorizonEdges.Num()-1]->Next->Vertex)
					{
						HorizonEdges.Add(Edge);
					}
					else
					{
#if DEBUG_HULL_GENERATION
						UE_LOGF(LogChaos, VeryVerbose, "\tNON VALID EDGE LOOP - internal horizon edge detected [%d - %d] - skipping ", Edge->Vertex, Edge->Next->Vertex);
#endif 
					}
				}
			}

#if DEBUG_HULL_GENERATION
#if DEBUG_HULL_GENERATION_BUILDHORIZON_TO_OBJ
			UE_LOGF(LogChaos, VeryVerbose, "# ======================================================");
			const FVec3Type ConflictPos = InVertices[ConflictV->Vertex];
			UE_LOGF(LogChaos, VeryVerbose, "# BUILD_HORIZON - Conflict Vertex = %d (%f %f %f)", ConflictV->Vertex, ConflictPos.X, ConflictPos.Y, ConflictPos.Z);
			UE_LOGF(LogChaos, VeryVerbose, "# ------------------------------------------------------");
			for (TSet<FConvexFace*>::TConstIterator SetIt(Processed); SetIt; ++SetIt)
			{
				const FConvexFace* Face = *SetIt;
				const FVector P1 = InVertices[Face->FirstEdge->Prev->Vertex];
				const FVector P2 = InVertices[Face->FirstEdge->Next->Vertex];
				const FVector P3 = InVertices[Face->FirstEdge->Vertex];
				UE_LOGF(LogChaos, VeryVerbose, "v %f %f %f", P1.X, P1.Y, P1.Z);
				UE_LOGF(LogChaos, VeryVerbose, "v %f %f %f", P2.X, P2.Y, P2.Z);
				UE_LOGF(LogChaos, VeryVerbose, "v %f %f %f", P3.X, P3.Y, P3.Z);
				UE_LOGF(LogChaos, VeryVerbose, "f -3 -2 -1");
			}
#endif

			FString HorizonString("> Final Horizon: (");
			for (const FHalfEdge* HorizonEdge : HorizonEdges)
			{
				HorizonString += FString::Printf(TEXT("%d (%d)"), HorizonEdge->Vertex, HorizonEdge->Next->Vertex);
			}
			HorizonString += TEXT(")");
			UE_LOGF(LogChaos, VeryVerbose, "%ls", *HorizonString);

			UE_LOGF(LogChaos, VeryVerbose, "Generate horizon - END");
#endif

		}

		static bool BuildFaces(FMemPool& Pool, const TArray<FVec3Type>& InVertices, const FHalfEdge* ConflictV, const TArray<FHalfEdge*>& HorizonEdges, const TArray<FConvexFace*>& OldFaces, TArray<FConvexFace*>& NewFaces)
		{
			//The HorizonEdges are in CCW order. We must make new faces and edges to join from ConflictV to these edges
			if (!(HorizonEdges.Num() >= 3)) // TODO: previously this was a check(), but it sometimes failed; can we fix things so this always holds?
			{
				return false;
			}
			NewFaces.Reserve(HorizonEdges.Num());
			FHalfEdge* PrevEdge = nullptr;
			for(int32 HorizonIdx = 0; HorizonIdx < HorizonEdges.Num(); ++HorizonIdx)
			{
				FHalfEdge* OriginalEdge = HorizonEdges[HorizonIdx];
				FHalfEdge* NewHorizonEdge = Pool.AllocHalfEdge(OriginalEdge->Vertex);
				NewHorizonEdge->Twin = OriginalEdge->Twin; //swap edges
				NewHorizonEdge->Twin->Twin = NewHorizonEdge;
				FHalfEdge* HorizonNext = Pool.AllocHalfEdge(OriginalEdge->Next->Vertex);
				// TODO: previously this was a check(), but it sometimes failed; can we fix things so this always holds?
				if (!(HorizonNext->Vertex == HorizonEdges[(HorizonIdx + 1) % HorizonEdges.Num()]->Vertex)) //should be ordered properly
				{
					return false;
				}
				FHalfEdge* V = Pool.AllocHalfEdge(ConflictV->Vertex);
				V->Twin = PrevEdge;
				if(PrevEdge)
				{
					PrevEdge->Twin = V;
				}
				PrevEdge = HorizonNext;

				//link new faces together
				FConvexFace* NewFace = CreateFace(Pool, InVertices, NewHorizonEdge, HorizonNext, V);
				if(NewFaces.Num() > 0)
				{
					NewFace->Prev = NewFaces[NewFaces.Num() - 1];
					NewFaces[NewFaces.Num() - 1]->Next = NewFace;
				}
				else
				{
					NewFace->Prev = nullptr;
				}
				NewFaces.Add(NewFace);
			}

			// TODO: previously this was a check(); can we determine if this always hold and switch it back if so?
			if (!PrevEdge)
			{
				return false;
			}
			NewFaces[0]->FirstEdge->Prev->Twin = PrevEdge;
			PrevEdge->Twin = NewFaces[0]->FirstEdge->Prev;
			NewFaces[NewFaces.Num() - 1]->Next = nullptr;

			//redistribute conflict list
			for(FConvexFace* OldFace : OldFaces)
			{
				StealConflictList(Pool, InVertices, OldFace->ConflictList, &NewFaces[0], NewFaces.Num());
			}

			//insert all new faces after conflict vertex face
			FConvexFace* OldFace = ConflictV->Face;
			FConvexFace* StartFace = NewFaces[0];
			FConvexFace* EndFace = NewFaces[NewFaces.Num() - 1];
			if(OldFace->Next)
			{
				OldFace->Next->Prev = EndFace;
			}
			EndFace->Next = OldFace->Next;
			OldFace->Next = StartFace;
			StartFace->Prev = OldFace;

			return true;
		}

		static bool AddVertex(FMemPool& Pool, const TArray<FVec3Type>& InVertices, FHalfEdge* ConflictV, const Params& InParams)
		{
			UE_CLOGF(DEBUG_HULL_GENERATION, LogChaos, VeryVerbose, "Adding Vertex %d", ConflictV->Vertex);

			TArray<FHalfEdge*> HorizonEdges;
			TArray<FConvexFace*> FacesToDelete;
			BuildHorizon(InVertices, ConflictV, HorizonEdges, FacesToDelete, InParams);

			TArray<FConvexFace*> NewFaces;
			if (!ensure(BuildFaces(Pool, InVertices, ConflictV, HorizonEdges, FacesToDelete, NewFaces)))
			{
				return false;
			}

#if DEBUG_HULL_GENERATION
			FString NewFaceString("\tNew Faces: ");
			for(const FConvexFace* Face : NewFaces)
			{
				NewFaceString += FString::Printf(TEXT("(%d %d %d) "), Face->FirstEdge->Vertex, Face->FirstEdge->Next->Vertex, Face->FirstEdge->Prev->Vertex);
			}
			UE_LOGF(LogChaos, VeryVerbose, "%ls", *NewFaceString);

			FString DeleteFaceString("\tDelete Faces: ");
			for(const FConvexFace* Face : FacesToDelete)
			{
				DeleteFaceString += FString::Printf(TEXT("(%d %d %d) "), Face->FirstEdge->Vertex, Face->FirstEdge->Next->Vertex, Face->FirstEdge->Prev->Vertex);
			}
			UE_LOGF(LogChaos, VeryVerbose, "%ls", *DeleteFaceString);
#endif

			for(FConvexFace* Face : FacesToDelete)
			{
				FHalfEdge* Edge = Face->FirstEdge;
				do
				{
					FHalfEdge* Next = Edge->Next;
					Pool.FreeHalfEdge(Next);
					Edge = Next;
				} while(Edge != Face->FirstEdge);
				if(Face->Prev)
				{
					Face->Prev->Next = Face->Next;
				}
				if(Face->Next)
				{
					Face->Next->Prev = Face->Prev;
				}
				Pool.FreeConvexFace(Face);
			}

			//todo(ocohen): need to explicitly test for merge failures. Coplaner, nonconvex, etc...
			//getting this in as is for now to unblock other systems

			return true;
		}

	};
}

