// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ChaosArchive.h"
#include "UObject/ExternalPhysicsCustomObjectVersion.h"

// Supported flags for Chaos filter, shares some flags with EPhysXFilterDataFlags
// Shared flags should be kept in sync until unified. @see EPhysXFilterDataFlags
// #TODO unify filter builder with this flag list

namespace Chaos
{
	enum class EFilterFlags : uint8
	{
		None					= 0b00000000,
		SimpleCollision			= 0b00000001,	// The shape is used for simple collision
		ComplexCollision		= 0b00000010,	// The shape is used for complex (trimesh) collision
		CCD						= 0b00000100,	// Unused - present for compatibility. CCD handled per-particle in Chaos
		ContactNotify			= 0b00001000,	// Whether collisions with this shape should be reported back to the game thread
		StaticShape				= 0b00010000,	// Unused - present for compatibility
		ModifyContacts			= 0b00100000,	// Unused - present for compatibility, whether to allow contact modification, handled in Chaos callbacks now
		KinematicKinematicPairs	= 0b01000000,	// Unused - present for compatibility, whether to generate KK pairs, Chaos never generates KK pairs
		All						= 0xFF,
	};
	ENUM_CLASS_FLAGS(EFilterFlags);

	inline const TCHAR* LexToString(EFilterFlags FilterFlag)
	{
		switch (FilterFlag)
		{
			case EFilterFlags::None: return TEXT("None");
			case EFilterFlags::SimpleCollision: return TEXT("SimpleCollision");
			case EFilterFlags::ComplexCollision: return TEXT("ComplexCollision");
			case EFilterFlags::CCD: return TEXT("CCD");
			case EFilterFlags::ContactNotify: return TEXT("ContactNotify");
			case EFilterFlags::StaticShape: return TEXT("StaticShape");
			case EFilterFlags::ModifyContacts: return TEXT("ModifyContacts");
			case EFilterFlags::KinematicKinematicPairs: return TEXT("KinematicKinematicPairs");
			case EFilterFlags::All: return TEXT("All");
			default: return TEXT("Invalid");
		}
	}
}

struct FCollisionFilterData
{
	uint32 Word0;
	uint32 Word1;
	uint32 Word2;
	uint32 Word3;

	FORCEINLINE FCollisionFilterData()
	{
		Word0 = Word1 = Word2 = Word3 = 0;
	}

	bool HasFlag(Chaos::EFilterFlags InFlag) const
	{
		const uint32 FilterFlags = (Word3 & 0xFFFFFF);
		return FilterFlags & static_cast<uint32>(InFlag);
	}

	friend inline bool operator!=(const FCollisionFilterData& A, const FCollisionFilterData& B)
	{
		return A.Word0!=B.Word0 || A.Word1!=B.Word1 || A.Word2!=B.Word2 || A.Word3!=B.Word3;
	}
};

inline Chaos::FChaosArchive& operator<<(Chaos::FChaosArchive& Ar, FCollisionFilterData& Filter)
{
	Ar << Filter.Word0 << Filter.Word1 << Filter.Word2 << Filter.Word3;
	return Ar;
}

namespace Chaos::Filter
{
	struct FCombinedShapeFilterData;
	struct FQueryFilterData;

	enum class ENarrowFilterResult
	{
		None = 0,
		Overlap = 1,
		Block = 2,
	};
	enum class EBroadFilterResult
	{
		// The object should be skipped from further processing.
		Reject,
		// The object should go through the rest of the detection pipeline.
		Accept,
	};

	struct FInstanceData
	{
		FInstanceData() = default;
		CHAOS_API FInstanceData(const uint32 InOwnerId, const uint32 ComponentId);

		bool operator==(const FInstanceData&) const = default;
		bool operator!=(const FInstanceData&) const = default;

		CHAOS_API bool IsValid() const;

		UE_DEPRECATED(5.8, "Use GetOwnerId instead") CHAOS_API uint32 GetActorId() const;
		UE_DEPRECATED(5.8, "Use SetOwnerId instead") CHAOS_API void SetActorId(const uint32 InActorId);
		CHAOS_API uint32 GetOwnerId() const;
		CHAOS_API void SetOwnerId(const uint32 InOwnerId);

		UE_INTERNAL CHAOS_API uint32 GetComponentId() const;
		UE_INTERNAL CHAOS_API void SetComponentId(const uint32 InComponentId);

		CHAOS_API void Serialize(FChaosArchive& Ar);
		CHAOS_API FString ToString() const;

	private:
		friend class FShapeFilterBuilder;
		friend FCombinedShapeFilterData;

		uint32 OwnerId = 0;
		uint32 ComponentId = 0;
	};

	struct FShapeFilterData
	{
		FShapeFilterData() = default;

		bool operator==(const FShapeFilterData&) const = default;
		bool operator!=(const FShapeFilterData&) const = default;

		CHAOS_API bool IsValid() const;
		CHAOS_API bool IsSimValid() const;

		CHAOS_API EFilterFlags GetFlags() const;
		CHAOS_API void SetFlags(EFilterFlags InFlags);
		CHAOS_API bool HasFlag(EFilterFlags InFlag) const;

		CHAOS_API uint8 GetMaskFilter() const;
		CHAOS_API void SetMaskFilter(uint8 MaskFilter);

		CHAOS_API uint8 GetCollisionChannelIndex() const;
		CHAOS_API uint64 GetCollisionChannelMask() const;
		CHAOS_API bool IsCollisionChannelSet(const uint32 ChannelIndex) const;

		CHAOS_API uint64 GetBlockChannels() const;
		CHAOS_API uint64 GetOverlapChannels() const;
		UE_DEPRECATED(5.8, "Use GetBlockChannels")
		CHAOS_API uint64 GetQueryBlockChannels() const;
		UE_DEPRECATED(5.8, "Use GetOverlapChannels")
		CHAOS_API uint64 GetQueryOverlapChannels() const;
		UE_DEPRECATED(5.8, "Use GetBlockChannels")
		CHAOS_API uint64 GetSimBlockChannels() const;

		// Filter API
		CHAOS_API ENarrowFilterResult NarrowFilter(const FShapeFilterData& OtherShape) const;

		CHAOS_API void Serialize(FChaosArchive& Ar);
		CHAOS_API FString ToString() const;

		UE_INTERNAL uint32 GetLegacyWord3() const;

		UE_INTERNAL void Store(FCollisionFilterData& QueryData, FCollisionFilterData& SimData) const;
		UE_INTERNAL void Load(const FCollisionFilterData& QueryData, const FCollisionFilterData& SimData);

	private:
		friend class FShapeFilterBuilder;
		friend FQueryFilterData;
		friend FCombinedShapeFilterData;
		friend struct FShapeUnionFilterData;

		uint64 BlockChannels = 0;
		uint64 OverlapChannels = 0;
		uint32 Word3 = 0;
	};

	struct FShapeUnionFilterData
	{
		FShapeUnionFilterData() = default;

		bool operator==(const FShapeUnionFilterData&) const = default;
		bool operator!=(const FShapeUnionFilterData&) const = default;

		CHAOS_API void Combine(const FShapeFilterData& ShapeFilter, bool bQueryEnabled = true, bool bSimEnabled = true);

		// Filter API
		CHAOS_API EBroadFilterResult BroadFilter(const FShapeUnionFilterData& OtherFilter) const;
		CHAOS_API EBroadFilterResult BroadFilter(const FQueryFilterData& QueryFilter) const;

	private:
		friend class FShapeFilterBuilder;
		uint64 QueryBlockChannels = 0;
		uint64 QueryOverlapChannels = 0;
		uint64 SimBlockChannels = 0;
		uint64 SimChannelMask = 0;
	};

	struct FQueryFilterData
	{
		enum EQueryType
		{
			Channel = 0,
			ObjectType,
		};

		FQueryFilterData() = default;

		bool operator==(const FQueryFilterData&) const = default;
		bool operator!=(const FQueryFilterData&) const = default;

		CHAOS_API bool IsValid() const;

		CHAOS_API EQueryType GetQueryType() const;

		CHAOS_API EFilterFlags GetFlags() const;
		CHAOS_API void SetFlags(EFilterFlags InFlags);
		CHAOS_API bool HasFlag(EFilterFlags InFlag) const;

		CHAOS_API uint8 GetIgnoreMask() const;

		// Channel API
		CHAOS_API uint8 GetCollisionChannelIndex() const;
		CHAOS_API uint64 GetCollisionChannelMask() const;
		CHAOS_API bool IsCollisionChannelSet(const uint32 ChannelIndex) const;
		CHAOS_API uint64 GetBlockChannels() const;
		CHAOS_API uint64 GetOverlapChannels() const;

		// ObjectType API
		CHAOS_API uint64 GetObjectTypesToQueryMask() const;
		CHAOS_API bool IsMultiQuery() const;

		// Filter API
		CHAOS_API ENarrowFilterResult NarrowFilter(const FShapeFilterData& ShapeFilter) const;

		CHAOS_API FString ToString() const;

	private:
		friend class FQueryFilterBuilder;
		friend class FQueryObjectFilterBuilder;
		friend class FQueryTraceFilterBuilder;

		ENarrowFilterResult ChannelTypeNarrowFilter(const FShapeFilterData& ShapeFilter) const;
		ENarrowFilterResult ObjectTypeNarrowFilter(const FShapeFilterData& ShapeFilter) const;

		uint32 Word0 = 0;
		uint32 Word3 = 0;
		// Note: These are the old word names, but they're arranged for memory packing
		uint64 Word1 = 0;
		uint64 Word2 = 0;
	};

	struct FCombinedShapeFilterData
	{
		FCombinedShapeFilterData() = default;
		CHAOS_API FCombinedShapeFilterData(const FShapeFilterData& InShapeFilter, const FInstanceData& InInstanceData);

		bool operator==(const FCombinedShapeFilterData&) const = default;
		bool operator!=(const FCombinedShapeFilterData&) const = default;

		CHAOS_API const FInstanceData& GetInstanceData() const;
		CHAOS_API void SetInstanceData(const FInstanceData& InData);

		CHAOS_API const FShapeFilterData& GetShapeFilterData() const;
		CHAOS_API void SetShapeFilterData(const FShapeFilterData& InData);

		CHAOS_API bool IsValid() const;
		UE_INTERNAL CHAOS_API bool IsSimValid() const;

	private:
		friend class FShapeFilterBuilder;
		FShapeFilterData ShapeFilterData;
		FInstanceData InstanceData;
	};

	class UE_INTERNAL FShapeFilterBuilder
	{
	public:
		FShapeFilterBuilder() = default;
		CHAOS_API FShapeFilterBuilder(const FShapeFilterData& InFilterData);

		// TODO @ JoshD: Eventually deprecated and replace with SetChannelMask
		CHAOS_API FShapeFilterBuilder& SetCollisionChannelIndex(const uint8 ChannelIndex);
		CHAOS_API FShapeFilterBuilder& SetBlockChannelMask(const uint64 BlockMask);
		CHAOS_API FShapeFilterBuilder& SetOverlapChannelMask(const uint64 OverlapMask);
		CHAOS_API FShapeFilterBuilder& SetMaskFilter(const uint8 InMaskFilter);
		CHAOS_API FShapeFilterBuilder& SetFilterFlags(const EFilterFlags InFilterFlags);
		CHAOS_API FShapeFilterBuilder& SetFilterFlags(const EFilterFlags InFilterFlags, bool bEnabled);

		CHAOS_API FShapeFilterData Build() const;
		CHAOS_API static FShapeFilterData BuildBlockAll(const EFilterFlags InFilterFlags = EFilterFlags::None);

		UE_INTERNAL CHAOS_API static FShapeFilterData BuildFromLegacySimFilter(const FCollisionFilterData& SimFilter);
		UE_INTERNAL CHAOS_API static FCombinedShapeFilterData BuildFromLegacyShapeFilter(const FCollisionFilterData& QueryFilterData, const FCollisionFilterData& SimFilterData);

		UE_INTERNAL CHAOS_API static void SetLegacyShapeQueryFilter(FCombinedShapeFilterData& CombinedShapeFilterData, const FCollisionFilterData& QueryFilterData);
		UE_INTERNAL CHAOS_API static void SetLegacyShapeSimFilter(FCombinedShapeFilterData& CombinedShapeFilterData, const FCollisionFilterData& SimFilterData);

		UE_INTERNAL CHAOS_API static void GetLegacyShapeFilter(const FCombinedShapeFilterData& CombinedShapeFilterData, FCollisionFilterData& OutQueryFilterData, FCollisionFilterData& OutSimFilterData);
		UE_INTERNAL CHAOS_API static FCollisionFilterData GetLegacyShapeQueryFilter(const FCombinedShapeFilterData& CombinedShapeFilterData);
		UE_INTERNAL CHAOS_API static FCollisionFilterData GetLegacyShapeSimFilter(const FCombinedShapeFilterData& CombinedShapeFilterData);

		UE_INTERNAL static uint32 GetWord3FromNewSimFilter(const uint32 SimWord3, const uint32 QueryWord3);
		UE_INTERNAL static uint32 GetWord3FromNewQueryFilter(const uint32 QueryWord3, const uint32 SimWord3);

		UE_INTERNAL CHAOS_API static FCombinedShapeFilterData LoadFromLegacySerialization(const FCollisionFilterData& QueryFilterData, const FCollisionFilterData& SimFilterData);

	private:
		static void BuildFromLegacyShapeFilter(const FCollisionFilterData& QueryFilterData, const FCollisionFilterData& SimFilterData, const uint32 BlockChannels, const uint32 Word3, FCombinedShapeFilterData& OutCombinedShapeFilterData);

		FShapeFilterData ShapeFilterData;
	};

	class UE_INTERNAL FQueryFilterBuilder
	{
	public:
		UE_INTERNAL CHAOS_API static FQueryFilterData BuildFromLegacyQueryFilter(const FCollisionFilterData& QueryFilterData);
		UE_INTERNAL CHAOS_API static FCollisionFilterData GetLegacyQueryFilter(const FQueryFilterData& QueryFilterData);
	private:
	};

	class UE_INTERNAL FQueryObjectFilterBuilder
	{
	public:
		CHAOS_API FQueryObjectFilterBuilder();
		CHAOS_API FQueryObjectFilterBuilder(const FQueryFilterData& InFilter);

		CHAOS_API FQueryObjectFilterBuilder& SetObjectTypes(uint64 ObjectTypes);
		CHAOS_API FQueryObjectFilterBuilder& SetMultiQuery(bool bMultiQuery);
		CHAOS_API FQueryObjectFilterBuilder& SetMaskFilter(const uint8 InMaskFilter);
		CHAOS_API FQueryObjectFilterBuilder& SetFilterFlags(const EFilterFlags InFilterFlags);
		CHAOS_API FQueryObjectFilterBuilder& SetFilterFlags(const EFilterFlags InFilterFlags, bool bEnabled);

		CHAOS_API FQueryFilterData Build() const;
		CHAOS_API static FQueryFilterData BuildBlockAll();

	private:
		FQueryFilterData QueryFilterData;
	};

	class UE_INTERNAL FQueryTraceFilterBuilder
	{
	public:
		CHAOS_API FQueryTraceFilterBuilder();
		CHAOS_API FQueryTraceFilterBuilder(const FQueryFilterData& InFilter);

		// TODO @ JoshD: Eventually deprecated and replace with SetChannelMask
		CHAOS_API FQueryTraceFilterBuilder& SetCollisionChannelIndex(const uint8 CollisionChannelIndex);
		CHAOS_API FQueryTraceFilterBuilder& SetBlockChannelMask(const uint64 BlockChannelMask);
		CHAOS_API FQueryTraceFilterBuilder& SetOverlapChannelMask(const uint64 OverlapChannelMask);
		CHAOS_API FQueryTraceFilterBuilder& SetMaskFilter(const uint8 InMaskFilter);
		CHAOS_API FQueryTraceFilterBuilder& SetFilterFlags(const EFilterFlags InFilterFlags);
		CHAOS_API FQueryTraceFilterBuilder& SetFilterFlags(const EFilterFlags InFilterFlags, bool bEnabled);

		CHAOS_API FQueryFilterData Build() const;
		CHAOS_API static FQueryFilterData BuildOverlapAll(const EFilterFlags InFilterFlags = EFilterFlags::None);

	private:
		FQueryFilterData QueryFilterData;
	};
} // namespace Chaos::Filter

inline Chaos::FChaosArchive& operator<<(Chaos::FChaosArchive& Ar, Chaos::Filter::FInstanceData& Data)
{
	Data.Serialize(Ar);
	return Ar;
}

inline Chaos::FChaosArchive& operator<<(Chaos::FChaosArchive& Ar, Chaos::Filter::FShapeFilterData& Data)
{
	Data.Serialize(Ar);
	return Ar;
}
