// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/CollisionResolutionTypes.h"
#include "Chaos/CollisionResolutionUtil.h"
#include "Chaos/Collision/PBDCollisionConstraint.h"
#include "Chaos/ParticleHandleFwd.h"


namespace Chaos
{
	template <typename T, int d>
	class TSphere;

	class FCapsule;

	class FConvex;

	template <typename T, int d>
	class TPlane;

	class FBVHParticles;
	class FCollisionContext;
	class FHeightField;
	class FImplicitObject;
	class FPBDCollisionConstraint;
	class FTriangleMeshImplicitObject;


	namespace Collisions
	{

		//
		// Constraint API
		//

		/**
		 * @brief Update the contact manifold on the constraint
		 * @note Transforms are shape world-space transforms (not particle transforms)
		*/
		void CHAOS_API UpdateConstraint(FPBDCollisionConstraint& Constraint, const FRigidTransform3& ShapeWorldTransform0, const FRigidTransform3& ShapeWorldTransform1, const FReal Dt);

		/**
		 * @brief Update the contact manifold on the constraint
		 * @note Transforms are shape world-space transforms (not particle transforms) at the start of the sweep. The end of the sweep are the transforms stored in the constraint as ShapeWorldTransform0/1
		*/
		bool CHAOS_API UpdateConstraintSwept(FPBDCollisionConstraint& Constraint, const FRigidTransform3& ShapeStartWorldTransform0, const FRigidTransform3& ShapeStartWorldTransform1, const FReal Dt);

		/**
		 * @brief Determine the shape pair type for use in UpdateConstraints
		*/
		EContactShapesType CHAOS_API CalculateShapePairType(const FGeometryParticleHandle* Particle0, const FImplicitObject* Implicit0, const FGeometryParticleHandle* Particle1, const FImplicitObject* Implicit1, bool& bOutSwap);

		/**
		 * @brief Whether CCD should be enabled for a contact given the current particle velocities etc
		*/
		bool CHAOS_API ShouldUseCCD(const FGeometryParticleHandle* Particle0, const FVec3& DeltaX0, const FGeometryParticleHandle* Particle1, const FVec3& DeltaX1, FVec3& Dir, FReal& Length);

		// @todo(chaos): this should not be exposed but is currently used in tests
		void UpdateLevelsetLevelsetConstraint(const FRigidTransform3& WorldTransform0, const FRigidTransform3& WorldTransform1, const FReal Dt, FPBDCollisionConstraint& Constraint);

		//
		//
		// DEPRECATED STUFF
		//
		//

		UE_DEPRECATED(5.4, "No longer supported")
		EImplicitObjectType CHAOS_API GetImplicitCollisionType(const FGeometryParticleHandle* Particle, const FImplicitObject* Implicit);

		UE_DEPRECATED(5.4, "No longer supported")
		inline void CHAOS_API ResetChaosCollisionCounters() {}

		UE_DEPRECATED(5.3, "No longer supported")
		inline void CHAOS_API ConstructConstraints(
			TGeometryParticleHandle<FReal, 3>* Particle0, 
			TGeometryParticleHandle<FReal, 3>* Particle1, 
			const FImplicitObject* Implicit0, 
			const FPerShapeData* Shape0, 
			const FBVHParticles* Simplicial0, 
			const int32 ImplicitID0,
			const FImplicitObject* Implicit1,
			const FPerShapeData* Shape1, 
			const FBVHParticles* Simplicial1, 
			const int32 ImplicitID1,
			const FRigidTransform3& ParticleWorldTransform0,
			const FRigidTransform3& Transform0, 
			const FRigidTransform3& ParticleWorldTransform1, 
			const FRigidTransform3& Transform1, 
			const FReal CullDistance, 
			const FReal Dt, 
			const bool bEnableSweep, 
			const FCollisionContext& Context) {}

		UE_DEPRECATED(5.3, "No longer supported")
		inline void CHAOS_API ConstructConstraints(TGeometryParticleHandle<FReal, 3>* Particle0, TGeometryParticleHandle<FReal, 3>* Particle1, const FImplicitObject* Implicit0, const FPerShapeData* Shape0, const FBVHParticles* Simplicial0, const FImplicitObject* Implicit1, const FPerShapeData* Shape1, const FBVHParticles* Simplicial1, const FRigidTransform3& ParticleWorldTransform0, const FRigidTransform3& Transform0, const FRigidTransform3& ParticleWorldTransform1, const FRigidTransform3& Transform1, const FReal CullDistance, const FReal Dt, const bool bEnableSweep, const FCollisionContext& Context) {}

		// See Above
		UE_DEPRECATED(5.3, "No longer supported")
		inline void CHAOS_API ConstructConstraints(TGeometryParticleHandle<FReal, 3>* Particle0, TGeometryParticleHandle<FReal, 3>* Particle1, const FImplicitObject* Implicit0, const FPerShapeData* Shape0, const FBVHParticles* Simplicial0, const FImplicitObject* Implicit1, const FPerShapeData* Shape1, const FBVHParticles* Simplicial1, const FRigidTransform3& ParticleWorldTransform0, const FRigidTransform3& Transform0, const FRigidTransform3& ParticleWorldTransform1, const FRigidTransform3& Transform1, const FReal CullDistance, const FReal Dt, const FCollisionContext& Context) {}

		template<ECollisionUpdateType UpdateType>
		UE_DEPRECATED(5.3, "Use UpdateConstraint, but call Constraint.SetShapeWorldTransform first (see implementation)")
		void CHAOS_API UpdateConstraintFromGeometry(FPBDCollisionConstraint& Constraint, const FRigidTransform3& ParticleTransform0, const FRigidTransform3& ParticleTransform1, const FReal Dt);
	}
}
