// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/ObjectPool.h"
#include "Chaos/ParticleHandleFwd.h"

// Whether to use a pool for MidPhases and Collision Constraints
#define CHAOS_COLLISION_OBJECTPOOL_ENABLED 1
#define CHAOS_MIDPHASE_OBJECTPOOL_ENABLED 0

namespace Chaos
{
	namespace Private
	{
		template<typename T> class TConvexContactPoint;
		using FConvexContactPoint = TConvexContactPoint<FReal>;
		using FConvexContactPointf = TConvexContactPoint<FRealSingle>;

		enum class EConvexFeatureType : int8;
	}

	// Float and Double versions of ContactPoint
	template<typename T> class TContactPoint;
	using FContactPoint = TContactPoint<FReal>;
	using FContactPointf = TContactPoint<FRealSingle>;

	// C-Style array type
	template<typename T, int32 N> class TCArray;

	// A set of contact points for a manifold (standard size is up to 4 contacts)
	using FContactPointManifold = TCArray<FContactPoint, 4>;

	// A set of contact points for a manifold that may be more than 4 points
	using FContactPointLargeManifold = TArray<FContactPoint>;


	/** Specifies the type of work we should do*/
	enum class ECollisionUpdateType
	{
		Any,	//stop if we have at least one deep penetration. Does not compute location or normal
		Deepest	//find the deepest penetration. Compute location and normal
	};

	/** Return value of the collision modification callback */
	enum class ECollisionModifierResult
	{
		Unchanged,	/** No change to the collision */
		Modified,	/** Modified the collision, but want it to remain enabled */
		Disabled,	/** Collision should be disabled */
	};

	/** The shape types involved in a contact constraint. Used to look up the collision detection function */
	enum class EContactShapesType : int8
	{
		Unknown,
		SphereSphere,
		SphereCapsule,
		SphereBox,
		SphereConvex,
		SphereTriMesh,
		SphereHeightField,
		SpherePlane,
		CapsuleCapsule,
		CapsuleBox,
		CapsuleConvex,
		CapsuleTriMesh,
		CapsuleHeightField,
		BoxBox,
		BoxConvex,
		BoxTriMesh,
		BoxHeightField,
		BoxPlane,
		ConvexConvex,
		ConvexTriMesh,
		ConvexHeightField,
		GenericConvexConvex,
		LevelSetLevelSet,

		NumShapesTypes
	};

	/** How to treat collisions between two particles that both have OneWayInteraction enabled */
	enum class EOneWayInteractionPairCollisionMode
	{
		// Ignore collisions
		IgnoreCollision,

		// Collide using particle's regular shapes
		NormalCollision,

		// Collide as spheres
		SphereCollision,
	};

	//
	//
	//

	class FCollisionContext;
	class FGenericParticlePairMidPhase;
	class FParticlePairMidPhase;
	class FPBDCollisionConstraints;
	class FPBDCollisionConstraint;
	class FPBDCollisionConstraintHandle;
	class FPerShapeData;
	class FShapePairParticlePairMidPhase;

// Collision and MidPhases are stored in an ObjectPool (if CHAOS_COLLISION_OBJECTPOOL_ENABLED or CHAOS_MIDPHASE_OBJECTPOOL_ENABLED is set). 
// @see FCollisionConstraintAllocator
#if CHAOS_COLLISION_OBJECTPOOL_ENABLED 
	using FPBDCollisionConstraintPool = TObjectPool<FPBDCollisionConstraint>;
	using FPBDCollisionConstraintDeleter = TObjectPoolDeleter<FPBDCollisionConstraintPool>;
	using FPBDCollisionConstraintPtr = TUniquePtr<FPBDCollisionConstraint, FPBDCollisionConstraintDeleter>;
#else 
	using FPBDCollisionConstraintPtr = TUniquePtr<FPBDCollisionConstraint>;
#endif

#if CHAOS_MIDPHASE_OBJECTPOOL_ENABLED
	// Not yet supported due to awkwardness of defining a deleter for FParticlePairMidPhasePtr when we have pools of derived types.
	// We probably need a custom deleter (not TObjectPoolDeleter) and to store a pointer to the CollisionConstraintAllocator.
	static_assert(false, "CHAOS_MIDPHASE_OBJECTPOOL_ENABLED not supported yet")
	using FGenericParticlePairMidPhasePool = TObjectPool<FGenericParticlePairMidPhase>;
	using FShapePairParticlePairMidPhasePool = TObjectPool<FShapePairParticlePairMidPhase>;
	using FParticlePairMidPhaseDeleter = TObjectPoolDeleter<???>;
	using FParticlePairMidPhasePtr = TUniquePtr<FParticlePairMidPhase, FParticlePairMidPhaseDeleter>;
#else
	using FParticlePairMidPhasePtr = TUniquePtr<FParticlePairMidPhase>;
#endif
}
