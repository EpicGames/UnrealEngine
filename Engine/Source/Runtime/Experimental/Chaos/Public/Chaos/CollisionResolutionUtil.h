// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/CollisionResolutionTypes.h"
#include "Chaos/Pair.h"

namespace Chaos
{
	class FBVHParticles;
	class FImplicitObject;
	class FPBDCollisionConstraint;

	namespace Collisions
	{
		//
		// Utility
		//

		FRigidTransform3 
		GetTransform(const TGeometryParticleHandle<FReal, 3>* Particle);

		template<typename TRealType>
		inline TMatrix33<TRealType> ComputeFactorMatrix3(const TVec3<TRealType>& V, const TMatrix33<TRealType>& M, const TRealType Im)
		{
			// Rigid objects rotational contribution to the impulse.
			// Vx*M*VxT+Im
			check(Im > FLT_MIN);
			return TMatrix33<TRealType>(
				-V[2] * (-V[2] * M.M[1][1] + V[1] * M.M[2][1]) + V[1] * (-V[2] * M.M[2][1] + V[1] * M.M[2][2]) + Im,
				V[2] * (-V[2] * M.M[1][0] + V[1] * M.M[2][0]) - V[0] * (-V[2] * M.M[2][1] + V[1] * M.M[2][2]),
				-V[1] * (-V[2] * M.M[1][0] + V[1] * M.M[2][0]) + V[0] * (-V[2] * M.M[1][1] + V[1] * M.M[2][1]),
				V[2] * (V[2] * M.M[0][0] - V[0] * M.M[2][0]) - V[0] * (V[2] * M.M[2][0] - V[0] * M.M[2][2]) + Im,
				-V[1] * (V[2] * M.M[0][0] - V[0] * M.M[2][0]) + V[0] * (V[2] * M.M[1][0] - V[0] * M.M[2][1]),
				-V[1] * (-V[1] * M.M[0][0] + V[0] * M.M[1][0]) + V[0] * (-V[1] * M.M[1][0] + V[0] * M.M[1][1]) + Im);
		}

		/** Compute the diagonal part of the rigid objects rotational contribution
		 * @param V Position relative to the center of mass
		 * @param M Mass of the particle
		 * @param Im Inertia tensor of the particle
		 */
		template<typename TRealType>
		inline TVec3<TRealType> ComputeFactorDiagonal3(const TVec3<TRealType>& V, const TMatrix33<TRealType>& M, const TRealType Im)
		{
			// Rigid objects rotational contribution diagonal to the impulse.
			// Vx*M*VxT+Im
			check(Im > FLT_MIN);
			return TVec3<TRealType>(
				-V[2] * (-V[2] * M.M[1][1] + V[1] * M.M[2][1]) + V[1] * (-V[2] * M.M[2][1] + V[1] * M.M[2][2]) + Im,
				 V[2] * ( V[2] * M.M[0][0] - V[0] * M.M[2][0]) - V[0] * ( V[2] * M.M[2][0] - V[0] * M.M[2][2]) + Im,
				-V[1] * (-V[1] * M.M[0][0] + V[0] * M.M[1][0]) + V[0] * (-V[1] * M.M[1][0] + V[0] * M.M[1][1]) + Im);
		}

		FVec3
		GetEnergyClampedImpulse(const TPBDRigidParticleHandle<FReal, 3>* PBDRigid0, const TPBDRigidParticleHandle<FReal, 3>* PBDRigid1, const FVec3& Impulse, const FVec3& VectorToPoint1, const FVec3& VectorToPoint2, const FVec3& Velocity1, const FVec3& Velocity2);

		FVec3 GetEnergyClampedImpulse(
			const FVec3& Impulse,
			FReal InvM0,
			const FMatrix33& InvI0,
			FReal InvM1,
			const FMatrix33& InvI1,
			const FRotation3& Q0,
			const FVec3& V0,
			const FVec3& W0,
			const FRotation3& Q1,
			const FVec3& V1,
			const FVec3& W1,
			const FVec3& ContactOffset0,
			const FVec3& ContactOffset1,
			const FVec3& ContactVelocity0,
			const FVec3& ContactVelocity1);

		bool 
		SampleObjectNoNormal(const FImplicitObject& Object, const FRigidTransform3& ObjectTransform, const FRigidTransform3& SampleToObjectTransform, const FVec3& SampleParticle, FReal Thickness, FContactPoint& ContactPoint);

		bool 
		SampleObjectNormalAverageHelper(const FImplicitObject& Object, const FRigidTransform3& ObjectTransform, const FRigidTransform3& SampleToObjectTransform, const FVec3& SampleParticle, FReal Thickness, FReal& TotalThickness, FContactPoint& ContactPoint);

		template <ECollisionUpdateType UpdateType>
		FContactPoint
		SampleObject(const FImplicitObject& Object, const FRigidTransform3& ObjectTransform, const FBVHParticles& SampleParticles, const FRigidTransform3& SampleParticlesTransform, FReal Thickness);

		TArray<Pair<const FImplicitObject*, FRigidTransform3>> 
		FindRelevantShapes(const FImplicitObject* ParticleObj, const FRigidTransform3& ParticlesTM, const FImplicitObject& LevelsetObj, const FRigidTransform3& LevelsetTM, const FReal Thickness);

	}// Collisions

} // Chaos
