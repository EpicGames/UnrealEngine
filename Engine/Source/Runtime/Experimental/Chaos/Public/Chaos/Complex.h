// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// HEADER_UNIT_SKIP - Internal

namespace Chaos
{
class FComplex
{
  public:
	FComplex() {}
	FComplex(const FReal Real, const FReal Imaginary)
	    : MReal(Real), MImaginary(Imaginary) {}
	FComplex Conjugated() { return FComplex(MReal, -MImaginary); }
	FComplex operator*(const FReal Other) const { return FComplex(MReal * Other, MImaginary * Other); }
	FComplex operator+(const FComplex Other) const { return FComplex(MReal + Other.MReal, MImaginary + Other.MImaginary); }
	FComplex& operator-=(const FComplex Other)
	{
		MReal -= Other.MReal;
		MImaginary -= Other.MImaginary;
		return *this;
	}
	inline void MakeReal() { MImaginary = 0; }
	inline const FReal Real() const { return MReal; }
	inline const FReal Imaginary() const { return MImaginary; }

  private:
	FReal MReal;
	FReal MImaginary;
};

inline FComplex operator*(const FReal Other, const FComplex Complex)
{
	return Complex * Other;
}

template<class T>
using Complex UE_DEPRECATED(4.27, "Deprecated. this class is to be deleted, use FComplex instead") = FComplex;
}
