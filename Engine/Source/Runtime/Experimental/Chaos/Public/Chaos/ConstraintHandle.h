// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/Declares.h"
#include "Chaos/Vector.h"
#include "Chaos/ParticleHandleFwd.h"

// Set to 1 to help trap erros in the constraint management which typically manifests as a dangling constraint handle in the IslandManager
// 
// WARNING: Do not submit with either of these set to 1 !!
//
#ifndef CHAOS_CONSTRAINTHANDLE_DEBUG_ENABLED
#define CHAOS_CONSTRAINTHANDLE_DEBUG_ENABLED  0
#endif
#ifndef CHAOS_CONSTRAINTHANDLE_DEBUG_DETAILED_ENABLED
#define CHAOS_CONSTRAINTHANDLE_DEBUG_DETAILED_ENABLED (CHAOS_CONSTRAINTHANDLE_DEBUG_ENABLED && 0)
#endif

// This is a proxy for not allowing the above to be checked in - CIS should fail if it is left enabled
#if UE_BUILD_SHIPPING || UE_BUILD_TEST
static_assert(CHAOS_CONSTRAINTHANDLE_DEBUG_ENABLED == 0, "CHAOS_CONSTRAINTHANDLE_DEBUG_ENABLED should be 0");
static_assert(CHAOS_CONSTRAINTHANDLE_DEBUG_DETAILED_ENABLED == 0, "CHAOS_CONSTRAINTHANDLE_DEBUG_DETAILED_ENABLED should be 0");
#endif

namespace Chaos
{
	class FPBDConstraintContainer;
	class FPBDIndexedConstraintContainer;

	using FParticlePair = TVec2<FGeometryParticleHandle*>;
	using FConstParticlePair = TVec2<const FGeometryParticleHandle*>;

	namespace Private
	{
		class FPBDIslandConstraint;
	}

	/**
	 * @brief A type id for constraint handles to support safe up/down casting (including intermediate classes in the hierrachy)
	 *
	 * Every constraint handle must provide a StaticType() member which gives the constraint type
	 * name and base class chain.
	 * 
	 * Every constraint container must provide a GetConstraintHandleType() method to get the constraint 
	 * type for handles that reference the container.
	*/
	class FConstraintHandleTypeID
	{
	public:
		FConstraintHandleTypeID(const FName& InName, const FConstraintHandleTypeID* InBaseType = nullptr)
			: TypeName(InName)
			, BaseType(InBaseType)
		{
		}

		/**
		 * @brief An invalid constraint handle type for initialization and invalidation
		*/
		static const FConstraintHandleTypeID InvalidTypeID()
		{
			return FConstraintHandleTypeID(NAME_None);
		}

		/**
		 * @brief Whether this type can be cast to the specified type
		*/
		bool IsA(const FConstraintHandleTypeID& TypeID) const
		{
			if (TypeID.TypeName == TypeName)
			{
				return true;
			}
			if (BaseType != nullptr)
			{
				return BaseType->IsA(TypeID);
			}
			return false;
		}

		FString ToString() const
		{
			return TypeName.ToString();
		}

	private:
		FName TypeName;
		const FConstraintHandleTypeID* BaseType;
	};


	/**
	 * @brief Base class for constraint handles.
	 * 
	 * Constraints are referenced by handle in the constraint graph. 
	 * Constraint handles allow us to support different allocation and storage policies for constraints.
	 * E.g., heap-allocated constraints, array-based constraints etc.
	 * 
	 * @see FIndexedConstraintHandle, FIntrusiveConstraintHandle
	*/
	class FConstraintHandle
	{
	public:
		using FGeometryParticleHandle = TGeometryParticleHandle<FReal, 3>;

		FConstraintHandle() 
			: ConstraintContainer(nullptr)
			, GraphEdge(nullptr)
		{
		}

		FConstraintHandle(FPBDConstraintContainer* InContainer)
			: ConstraintContainer(InContainer)
			, GraphEdge(nullptr)
		{
		}

		virtual ~FConstraintHandle()
		{
			// Make sure we are not still in the graph
			check(GraphEdge == nullptr);
		}

		virtual bool IsValid() const
		{
			return (ConstraintContainer != nullptr);
		}

		FPBDConstraintContainer* GetContainer()
		{
			return ConstraintContainer;
		}

		const FPBDConstraintContainer* GetContainer() const 
		{
			return ConstraintContainer;
		}

		bool IsInConstraintGraph() const
		{
			return (GraphEdge != nullptr);
		}

		Private::FPBDIslandConstraint* GetConstraintGraphEdge() const
		{
			return GraphEdge;
		}

		// NOTE: Should only be called by the IslandManager
		void SetConstraintGraphEdge(Private::FPBDIslandConstraint* InEdge)
		{
			// Check for double add
			check((InEdge == nullptr) || (GraphEdge == nullptr));

			GraphEdge = InEdge;
		}

		virtual TVec2<FGeometryParticleHandle*> GetConstrainedParticles() const = 0;

		virtual void SetEnabled(bool InEnabled) = 0;

		virtual bool IsEnabled() const = 0;

		virtual bool IsProbe() const { return false; }

		// Does this constraint have the concept of sleep? (only really used for debug validation)
		virtual bool SupportsSleeping() const { return false; }

		// Can this constraint connect sleeping and active particles for partial island sleeping?
		virtual bool SupportsPartialIslandSleeping() const { return false; }

		virtual bool IsSleeping() const { return false; }
		virtual void SetIsSleeping(const bool bInIsSleeping) {}

		// Total net linear impulse applied by the velocity-based dynamics (VBD) solver (velocity iterations only).
		virtual FVec3f GetNetLinearImpulse() const { return FVec3f(0); }
		
		virtual bool WasAwakened() const { return false; }
		virtual void SetWasAwakened(const bool bInWasAwakened) {}

		// Implemented in ConstraintContainer.h
		int32 GetContainerId() const;

		// Implemented in ConstraintContainer.h
		template<typename T>  T* As();
		template<typename T>  const T* As() const;

		// For use when you absolutely know the type (asserted in non-shipping)
		template<typename T>  T* AsUnsafe() { check(As<T>() != nullptr); return static_cast<T*>(this); }
		template<typename T>  const T* AsUnsafe() const { check(As<T>() != nullptr); return static_cast<const T*>(this); }

		const FConstraintHandleTypeID& GetType() const;

		static const FConstraintHandleTypeID& StaticType()
		{
			static FConstraintHandleTypeID STypeID(TEXT("FConstraintHandle"), nullptr);
			return STypeID;
		}

		static const FConstraintHandleTypeID& InvalidType()
		{
			static FConstraintHandleTypeID STypeID(TEXT("InvalidConstraintHandle"), nullptr);
			return STypeID;
		}

		// Deprecated API
		UE_DEPRECATED(5.3, "Use GetConstraintGraphEdge") int32 GetConstraintGraphIndex() const { return INDEX_NONE; }
		UE_DEPRECATED(5.3, "Not supported") void SetConstraintGraphIndex(const int32 InIndex) const {}

	protected:
		friend class FPBDConstraintContainer;

		FPBDConstraintContainer* ConstraintContainer;
		
		Private::FPBDIslandConstraint* GraphEdge;
	};


	/**
	 * @brief Base class for constraints that are allocated at permanent memory addresses and inherit the handle.
	 *
	 * Intended for use by constraint types that are allocated on the heap or in a block allocator and therefore have a persistent
	 * address (as opposed to array-based containers where the array could relocate). The constraint class should inherit
	 * this handle class. This effectively eliminates the handle, reducing cache misses and allocations.
	*/
	class FIntrusiveConstraintHandle : public FConstraintHandle
	{
	public:
		FIntrusiveConstraintHandle()
			: FConstraintHandle()
		{
		}

		void SetContainer(FPBDConstraintContainer* InContainer)
		{
			ConstraintContainer = InContainer;
		}

		static const FConstraintHandleTypeID& StaticType()
		{
			static FConstraintHandleTypeID STypeID(TEXT("FIntrusiveConstraintHandle"), &FConstraintHandle::StaticType());
			return STypeID;
		}
	};


	/**
	 * @brief Base class for constraints that are allocated at permanent memory addresses and inherit the handle.
	 * 
	 * @see FIntrusiveConstraintHandle
	 * 
	 * @tparam T_CONSTRAINT The constraint type
	*/
	template<typename T_CONSTRAINT>
	class TIntrusiveConstraintHandle : public FIntrusiveConstraintHandle
	{
	public:
		using FConstraint = T_CONSTRAINT;

		TIntrusiveConstraintHandle()
			: FIntrusiveConstraintHandle()
		{
		}

		void SetContainer(FPBDConstraintContainer* InContainer)
		{
			ConstraintContainer = InContainer;
		}

		FConstraint* GetConstraint()
		{
			return static_cast<FConstraint*>(this);
		}

		const FConstraint* GetConstraint() const
		{
			return static_cast<const FConstraint*>(this);
		}
	};



	/**
	 * An allocator for constraint handles.
	 *
	 * @todo(ccaulfield): block allocator for handles, or support custom allocators in constraint containers.
	 */
	template<class T_CONTAINER>
	class TConstraintHandleAllocator
	{
	public:
		using FConstraintContainer = T_CONTAINER;
		using FConstraintContainerHandle = typename FConstraintContainer::FConstraintContainerHandle;

		FConstraintContainerHandle* AllocHandle(FConstraintContainer* ConstraintContainer, int32 ConstraintIndex) { return new FConstraintContainerHandle(ConstraintContainer, ConstraintIndex); }

		void FreeHandle(FConstraintContainerHandle* Handle) { delete Handle; }
	};


	/**
	 * @brief A debugging utility for tracking down dangling constraint issues
	 * This acts as a FConstraintHandle*, but caches some extra debug data useful in tracking
	 * down dangling pointer issues when they arise.
	*/
	class FConstraintHandleHolder
	{
	public:
		FConstraintHandleHolder()
			: Handle(nullptr)
		{
#if CHAOS_CONSTRAINTHANDLE_DEBUG_ENABLED
			InitDebugData();
#endif
		}

		FConstraintHandleHolder(FConstraintHandle* InHandle)
			: Handle(InHandle)
		{
#if CHAOS_CONSTRAINTHANDLE_DEBUG_ENABLED
			InitDebugData();
#endif
		}

		FConstraintHandle* operator->() const { return Handle; }
		operator FConstraintHandle* () const { return Handle; }
		FConstraintHandle* Get() const { return Handle; }

		friend uint32 GetTypeHash(const FConstraintHandleHolder& V)
		{
			return ::GetTypeHash(V.Handle);
		}

	private:
		FConstraintHandle* Handle;

#if CHAOS_CONSTRAINTHANDLE_DEBUG_ENABLED
	public:
		const FGeometryParticleHandle* GetParticle0() const { return Particles[0]; }
		const FGeometryParticleHandle* GetParticle1() const { return Particles[1]; }

	private:
		CHAOS_API void InitDebugData();

		const FConstraintHandleTypeID* ConstraintType;
		const FGeometryParticleHandle* Particles[2];
#endif
	};
}
