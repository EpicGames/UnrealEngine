// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "ParticleHandleFwd.h"
#include "Chaos/ImplicitFwd.h"

namespace
{
	// When a FContactPairModifierParticleRange is created to iterate over the contacts
	// of a particular particle, an array of relevant constraints is generated.
	//
	// So long as the number of constraints for this particle does not exceed this constant
	// value, no heap allocations will occur.
	constexpr int32 ParticleContactsStackSize = 16;
}

namespace Chaos
{
	class FImplicitObject;
	class FPBDCollisionConstraint;
	class FPBDCollisionConstraints;
	class FCollisionContactModifier;
	class FContactPairModifier;
	class FPerShapeData;
	class FShapeInstance;
	class FPBDCollisionConstraintContainerCookie;

	class FContactPairModifier
	{
	public:
		FContactPairModifier()
			: Constraint(nullptr)
			, Modifier(nullptr)
		{}

		FContactPairModifier(FPBDCollisionConstraint* InConstraint, FCollisionContactModifier& InModifier)
			: Constraint(InConstraint)
			, Modifier(&InModifier)
		{}

		/**
		* Disable all contact points for this pair of bodies.
		*/
		CHAOS_API void Disable();

		/**
		* Enables all contact points for this pair of bodies. This does nothing unless Disable() was previously called.
		*/
		CHAOS_API void Enable();

		/**
		* Convert the constraint between this pair of bodies into a probe. Collision callbacks will still occur if the
		* bodies collide (a contact occurring in contact modification does not mean a contact actually has occurred
		* yet, but that it may)
		*/
		CHAOS_API void ConvertToProbe();

		/**
		 * Convert the constraint from probe to a regular physical collision
		 */
		CHAOS_API void ConvertToNonProbe();

		/**
		 * Return true if this constraint is a probe type
		 */
		CHAOS_API bool GetIsProbe() const;

		/**
		* @return Number of contact points in constraint pair. ContactPointIdx must be below number of contacts.
		*/
		CHAOS_API int32 GetNumContacts() const;

		/**
		* @return Contact point index of point in manifold penetrating the deepest.
		*/
		CHAOS_API int32 GetDeepestContactIndex() const;

		/**
		* @return true if this contact is an initial contact, i.e., the pair of shapes in the contact were not touching last frame.
		*/
		CHAOS_API bool GetIsInitialContact() const;

		/**
		* Set whether this contact is considered an initial contact. This controls how the contact behaves with regard to the initial
		* overlap depenetration system.
		*/
		CHAOS_API void SetIsInitialContact(bool bIsInitialContact) const;

		/**
		* @return Geometry of specified particle in contact pair.
		*/
		CHAOS_API const FImplicitObject* GetContactGeometry(int32 ParticleIdx);

		/**
		* @return Transformation from geometry to world.
		*/
		CHAOS_API FRigidTransform3 GetShapeToWorld(int32 ParticleIdx) const;

		/**
		* @return separation of contact locations in direction of normal.
		*/
		CHAOS_API FReal GetSeparation(int32 ContactPointIdx) const;

		/**
		* @return desired separation of contact locations in direction of normal.
		*/
		CHAOS_API FReal GetTargetSeparation(int32 ContactPointIdx) const;

		/**
		* Set the desired separation at the contact point (negative for penetration).
		*/
		CHAOS_API void ModifyTargetSeparation(FReal TargetSeparation, int32 ContactPointIdx);

		/**
		* @brief Get the world-space contact normal.
		* @note The contact normal always points away from the second body.
		* E.g., a sphere lying on a flat ground could return a WorldNormal pointing up or down, depending on whether the
		* sphere is the first or second body in the constraint.
		* @return Normal of contact in world space.
		*/
		CHAOS_API FVec3 GetWorldNormal(int32 ContactPointIdx) const;

		/**
		* Modify contact normal in world space. If modifying separation and normal, order of operations should be considered.
		* @note The contact normal should always point away from the second body.
		*/
		CHAOS_API void ModifyWorldNormal(const FVec3& Normal, int32 ContactPointIdx);

		/**
		* Get contact location on each body in world space.
		*/
		CHAOS_API void GetWorldContactLocations(int32 ContactPointIdx, FVec3& OutLocation0, FVec3& OutLocation1)  const;

		/**
		* @return Contact location. This is midpoint of each body's contact location.
		*/
		CHAOS_API FVec3 GetWorldContactLocation(int32 ContactPointIdx)  const;

		/**
		* Modify contact location of each body in world space. If modifying locations and separation, order of operations should be considered.
		*/
		CHAOS_API void ModifyWorldContactLocations(const FVec3& Location0, const FVec3& Location1, int32 ContactPointIdx);

		CHAOS_API void ModifyEnablePositionSolve(bool bEnable, int32 ContactPointIdx);
		CHAOS_API void ModifyEnableVelocitySolve(bool bEnable, int32 ContactPointIdx);
		CHAOS_API void ModifyEnableFriction(bool bEnable, int32 ContactPointIdx);

		/**
		* @return Restitution of contact.
		*/
		CHAOS_API FReal GetRestitution() const;

		/**
		* Modify restitution of contact. If object is moving slowly, RestitutionThreshold may need to be lowered if restitution is desired.
		*/
		CHAOS_API void ModifyRestitution(FReal Restitution);

		/**
		* @return Restitution threshold of contact. If velocity of object is below this threshold, no restitution is applied. 
		*/
		CHAOS_API FReal GetRestitutionThreshold() const;

		/**
		* Modify restitution threshold of contact. If velocity of object is below this threshold, no restitution is applied.
		*/
		CHAOS_API void ModifyRestitutionThreshold(FReal Restitution);

		/**
		* @return Dynamic friction coefficient of contact.
		*/
		CHAOS_API FReal GetDynamicFriction() const;

		/**
		* Modify dynamic friction coefficient of contact.
		*/
		CHAOS_API void ModifyDynamicFriction(FReal DynamicFriction);

		/**
		* @return Static friction coefficient of contact.
		*/
		CHAOS_API FReal GetStaticFriction() const;

		/**
		* Modify static friction coefficient of contact.
		*/
		CHAOS_API void ModifyStaticFriction(FReal StaticFriction);

		/**
		* @return Linear velocity of particle.
		*/
		CHAOS_API FVec3 GetParticleVelocity(int32 ParticleIdx) const;

		/**
		* Modify linear velocity. For simulated objects: Previous frame's position is modified to maintain PBD implicit velocity.
		*/
		CHAOS_API void ModifyParticleVelocity(FVec3 Velocity, int32 ParticleIdx);

		/**
		* @return Angular velocity of particle.
		*/
		CHAOS_API FVec3 GetParticleAngularVelocity(int32 ParticleIdx) const;

		/**
		* Modify angular velocity. For simulated objects: Previous frame's rotation is modified to maintain PBD implicit velocity.
		*/
		CHAOS_API void ModifyParticleAngularVelocity(FVec3 AngularVelocity, int32 ParticleIdx);

		/**
		* @return Position of particle. For simulated objects: this is PBD predicted position and not previous position before integrating movement.
		*/
		CHAOS_API FVec3 GetParticlePosition(int32 ParticleIdx) const;

		/**
		* For simulated objects: this sets PBD predicted position and not position before integrating movement.
		* @param bMaintainVelocity - if true, simulated object will update previous position to maintain implicit velocity. Otherwise changing position will change velocity.
		*/
		CHAOS_API void ModifyParticlePosition(FVec3 Position, bool bMaintainVelocity, int32 ParticleIdx);

		/**
		* @return Rotation of particle. For simulated objects: this is PBD predicted rotation and not previous rotation before integrating movement.
		*/
		CHAOS_API FRotation3 GetParticleRotation(int32 ParticleIdx) const;

		/**
		* For simulated objects: this sets PBD predicted rotation and not rotation before integrating movement.
		* @param bMaintainVelocity - if true, simulated object will update previous rotation to maintain implicit velocity. Otherwise changing rotation will change angular velocity.
		*/
		CHAOS_API void ModifyParticleRotation(FRotation3 Rotation, bool bMaintainVelocity, int32 ParticleIdx);

		/**
		* @return InvInertiaScale of particle.
		*/
		CHAOS_API FReal GetInvInertiaScale(int32 ParticleIdx) const;

		/**
		* Modify InvInertiaScale of particle. 0 prevents all rotation of particle due to collision, values > 1 decrease rotational inertia.
		*/
		CHAOS_API void ModifyInvInertiaScale(FReal InvInertiaScale, int32 ParticleIdx);

		/**
		* @return InvMassScale of particle.
		*/
		CHAOS_API FReal GetInvMassScale(int32 ParticleIdx) const;

		/**
		* Modify InvMassScale of particle. 0 gives particle infinite mass during collision, values > 1 decrease mass during collision.
		*/
		CHAOS_API void ModifyInvMassScale(FReal InvMassScale, int32 ParticleIdx);

		/*
		* Get the other particle in the constraint - if the input particle isn't in the constraint,
		* get null
		*/
		CHAOS_API FGeometryParticleHandle* GetOtherParticle(FGeometryParticleHandle* Particle) const;

		/*
		* Retrieve physics handles for particles in contact pair.
		*/
		CHAOS_API TVec2<FGeometryParticleHandle*> GetParticlePair() const;

		/*
		* Get shape pair from constraint.
		*/
		CHAOS_API TVec2<const FPerShapeData*> GetShapePair() const;

		/*
		* Get one of the particle's shapes.
		*/
		CHAOS_API const FShapeInstance* GetShape(int32 ParticleIdx) const;

		/*
		* Get one of the particle's implicit objects.
		*/
		CHAOS_API const FConstImplicitObjectRef GetImplicit(int32 ParticleIdx) const;

		/*
		* Whether this contact was generated from a CCD hit. Even if CCD is enabled for the bodies,
		* the hit may be a non-CCD contact if they are moving slowly.
		* NOTE: You can disable CCD in a MidPhase Modifier, but it is too late to do that here.
		*/
		CHAOS_API bool GetIsCCD() const;

		/*
		* Check to see if a contact point index is an edge contact.
		*/
		CHAOS_API bool IsEdgeContactPoint(int32 ContactPointIdx) const;

		/*
		* Check to see if a contact point index is disabled.
		*/
		CHAOS_API bool IsContactPointDisabled(int32 ContactPointIdx) const;

		/*
		* Set a contact point disabled.
		*/
		CHAOS_API void SetContactPointDisabled(int32 ContactPointIdx) const;

		/*
		* Add a new contact point. WorldPos0 and WorldPos1 are the world-space contact positions on each particle.
		* WorldNormal is the contact normal. The contact separation is implicitly (WorldPos0 - WorldPos1).WorldNormal.
		* If WorldPos0 and WorldPos1 are not aligned along WorldNormal, then friction will be applied to attempt to 
		* correct the tangential separation.
		*/
		CHAOS_API int32 AddContact(const FVec3& WorldPos0, const FVec3& WorldPos1, const FVec3& WorldNormal) const;

	private:

		/*
		* Get direct const access to the index of this constraint container cookie
		*/
		CHAOS_API const FPBDCollisionConstraintContainerCookie& GetConstraintContainerCookie() const;

		/**
		 * @brief Update cached shape transforms in the constraint after modifying particle positions
		*/
		CHAOS_API void UpdateConstraintShapeTransforms();

		FPBDCollisionConstraint* Constraint;
		FCollisionContactModifier* Modifier;

		friend class FVisitedContactPairsTracker;
	};

	class FContactPairModifierIterator
	{
	public:
		FContactPairModifierIterator() { SetToEnd(); }

		FContactPairModifierIterator(FCollisionContactModifier& InModifier)
			: ConstraintIdx(0)
			, Modifier(&InModifier)
			, PairModifier()
		{
			// Initialize modifier at idx 0 or move to end.
			SeekValidContact();
		}

		FContactPairModifier& operator*()
		{
			// Make sure we are not returning invalid modifier if at End.
			EnsureValid();

			return PairModifier;
		}

		FContactPairModifier* operator->()
		{
			// Make sure we are not returning invalid modifier if at End.
			EnsureValid();

			return &PairModifier;
		}

		FContactPairModifierIterator& operator++()
		{
			++ConstraintIdx;
			SeekValidContact();
			return *this;
		}

		explicit operator bool() const
		{
			return IsValid();
		}

		bool operator==(const FContactPairModifierIterator& Value) const
		{
			// All other data should match if index matches.
			return ConstraintIdx == Value.ConstraintIdx;
		}

		bool operator!=(const FContactPairModifierIterator& Value) const
		{
			return !(*this == Value);
		}

		bool IsValid() const
		{
			return ConstraintIdx != INDEX_NONE ;
		}

	private:

		void SetToEnd()
		{
			ConstraintIdx = INDEX_NONE;
			Modifier = nullptr;
		}

		void EnsureValid() const
		{
			ensure(IsValid());
		}

		// Moves to constraint at ConstraintIdx, or finds next valid constraint. Moves to end if no remaining valid constraints.
		CHAOS_API void SeekValidContact();

		int32 ConstraintIdx;
		FCollisionContactModifier* Modifier;
		FContactPairModifier PairModifier;
	};

	/*
	* Iterator for FContactPairModifierParticleRange
	*/
	class FContactPairModifierParticleRangeIterator
	{
	public:

		FContactPairModifier& operator*()
		{
			return PairModifier;
		}

		FContactPairModifier* operator->()
		{
			return &PairModifier;
		}

		FContactPairModifierParticleRangeIterator& operator++()
		{
			++Index;

			PairModifier
				= IsValid()
				? FContactPairModifier(Constraints[Index], *Modifier)
				: FContactPairModifier();

			return *this;
		}

		bool operator==(const FContactPairModifierParticleRangeIterator& Other)
		{
			return
				&Constraints == &Other.Constraints &&
				Index == Other.Index;
		}

		bool operator!=(const FContactPairModifierParticleRangeIterator& Other)
		{
			return !(*this == Other);
		}

		bool IsValid() const
		{
			return 0 <= Index && Index < Constraints.Num();
		}

	private:
		FContactPairModifierParticleRangeIterator(FCollisionContactModifier* InModifier, TArray<FPBDCollisionConstraint*, TInlineAllocator<ParticleContactsStackSize>>& InConstraints, int32 InIndex)
			: Modifier(InModifier)
			, Constraints(InConstraints)
			, Index(InIndex)
		{
			PairModifier
				= IsValid()
				? FContactPairModifier(Constraints[Index], *Modifier)
				: FContactPairModifier();
		}

		FCollisionContactModifier* Modifier;
		FContactPairModifier PairModifier;

		TArray<FPBDCollisionConstraint*, TInlineAllocator<ParticleContactsStackSize>>& Constraints;
		int32 Index;

		// Befriend the range object so that it can create iterators
		friend class FContactPairModifierParticleRange;
	};

	/*
	* Interface for iterating over a range of contacts for a particular particle
	*/
	class FContactPairModifierParticleRange
	{
	public:
		CHAOS_API FContactPairModifierParticleRangeIterator begin();
		CHAOS_API FContactPairModifierParticleRangeIterator end();

		FContactPairModifierParticleRange(FContactPairModifierParticleRange&& Other)
			: Modifier(Other.Modifier)
			, Particle(Other.Particle)
			, Constraints(MoveTemp(Other.Constraints))
		{ }

	private:
		FContactPairModifierParticleRange(FCollisionContactModifier* InModifier, FGeometryParticleHandle* InParticle);
		FCollisionContactModifier* Modifier;
		FGeometryParticleHandle* Particle;
		TArray<FPBDCollisionConstraint*, TInlineAllocator<ParticleContactsStackSize>> Constraints;

		// Befriend the modifier so that it can create ranges
		friend class FCollisionContactModifier;
	};

	/*
	* Utility for tracking visited contact pair modifiers so that clients can avoid processing duplicates
	*/
	class FVisitedContactPairsTracker
	{
	public:

		// If a contact pair was not already visited, mark it visited and return true. Otherwise return false.
		CHAOS_API bool Visit(const FContactPairModifier& ContactPair);

	private:
		FVisitedContactPairsTracker(const TArrayView<FPBDCollisionConstraint* const>& InConstraints)
			: Constraints(InConstraints)
			, VisitedContacts(false, Constraints.Num())
		{ }

		TArrayView<FPBDCollisionConstraint* const> Constraints;
		TBitArray<> VisitedContacts;

		// Befriend the modifier so that it can create these
		friend class FCollisionContactModifier;
	};

	/*
	*  Provides interface for iterating over modifiable contact pairs
	*/
	class FCollisionContactModifier
	{
	public:
		friend FContactPairModifier;
		friend FContactPairModifierIterator;
		friend FPBDCollisionConstraints; // Calls UpdateConstraintManifolds after callback.


		FCollisionContactModifier(const TArrayView<FPBDCollisionConstraint* const>& InConstraints, FReal InDt)
			: Constraints(InConstraints)
			, Dt(InDt)
		{}

		FContactPairModifierIterator Begin() { return FContactPairModifierIterator(*this); }
		FContactPairModifierIterator begin() { return FContactPairModifierIterator(*this); }

		FContactPairModifierIterator End() const { return FContactPairModifierIterator(); }
		FContactPairModifierIterator end() const { return FContactPairModifierIterator(); }

		CHAOS_API FContactPairModifierParticleRange GetContacts(FGeometryParticleHandle* Particle);

		CHAOS_API FVisitedContactPairsTracker MakeVisitedContactPairsTracker() const;

	private:
		CHAOS_API TArrayView<FPBDCollisionConstraint* const>& GetConstraints();
		CHAOS_API void DisableConstraint(FPBDCollisionConstraint& Constraint);
		CHAOS_API void EnableConstraint(FPBDCollisionConstraint& Constraint);

		// Turn this constraint into a probe. It will still generate hit events,
		// but will not produce impulses
		CHAOS_API void ConvertToProbeConstraint(FPBDCollisionConstraint& Constraint);

		// Turn this constraint from a probe into a regular constraint
		CHAOS_API void ConvertToNonProbeConstraint(FPBDCollisionConstraint& Constraint);

		CHAOS_API void MarkConstraintForManifoldUpdate(FPBDCollisionConstraint& Constraint);

		// Update manifolds of modified constraints.
		CHAOS_API void UpdateConstraintManifolds();

		TArrayView<FPBDCollisionConstraint* const> Constraints;

		// Constraints that should update manifold from contact points.
		TSet<FPBDCollisionConstraint*> NeedsManifoldUpdate;

		FReal Dt;
	};

	using FCollisionModifierCallback = TFunction<void(const FCollisionContactModifier&)>;
}
