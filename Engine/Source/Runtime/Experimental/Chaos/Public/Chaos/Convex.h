// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/ImplicitObject.h"
#include "Chaos/AABB.h"
#include "Chaos/ConvexStructureData.h"
#include "Chaos/MassProperties.h"
#include "CollisionConvexMesh.h"
#include "ChaosArchive.h"
#include "ChaosCheck.h"
#include "ChaosLog.h"
#include "UObject/ReleaseObjectVersion.h"
#include "UObject/PhysicsObjectVersion.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

namespace Chaos
{
	//
	// Note: While Convex technically supports a margin, the margin is typically a property of the
	// instance wrapper (ImplicitScaled, ImplicitTransformed, or ImplicitInstanced). Usually the
	// margin on the convex itself is zero.
	//
	class FConvex final : public FImplicitObject
	{
	public:
		using FImplicitObject::GetTypeName;

		using TType = FRealSingle;
		using FRealType = TType;
		using FVec3Type = TVec3<FRealType>;
		using FPlaneType = TPlaneConcrete<FRealType, 3>;
		using FAABB3Type = TAABB<FRealType, 3>;

		static constexpr unsigned D = 3;

		FConvex()
		    : FImplicitObject(EImplicitObject::IsConvex | EImplicitObject::HasBoundingBox, ImplicitObjectType::Convex)
			, Volume(0.f)
			, CenterOfMass(FVec3Type(0.f))
			, UnitMassInertiaTensor(1., 1., 1.)
			, RotationOfMass(FRotation3::Identity)
		{}
		FConvex(FConvex&& Other)
		    : FImplicitObject(EImplicitObject::IsConvex | EImplicitObject::HasBoundingBox, ImplicitObjectType::Convex)
			, Planes(MoveTemp(Other.Planes))
		    , Vertices(MoveTemp(Other.Vertices))
		    , LocalBoundingBox(MoveTemp(Other.LocalBoundingBox))
			, StructureData(MoveTemp(Other.StructureData))
			, Volume(MoveTemp(Other.Volume))
			, CenterOfMass(MoveTemp(Other.CenterOfMass))
			, UnitMassInertiaTensor(MoveTemp(Other.UnitMassInertiaTensor))
			, RotationOfMass(MoveTemp(Other.RotationOfMass))
		{}

		// NOTE: This constructor will result in approximate COM and volume calculations, since it does
		// not have face indices for surface particles.
		// NOTE: Convex constructed this way will not contain any structure data
		// @todo(chaos): Keep track of invalid state and ensure on volume or COM access?
		// @todo(chaos): Add plane vertex indices in the constructor and call CreateStructureData
		// @todo(chaos): Merge planes? Or assume the input is a good convex hull?
		UE_DEPRECATED(4.27, "Use the constructor version with the face indices.")
		FConvex(TArray<FPlaneType>&& InPlanes, TArray<FVec3Type>&& InVertices)
		    : FImplicitObject(EImplicitObject::IsConvex | EImplicitObject::HasBoundingBox, ImplicitObjectType::Convex)
			, Planes(MoveTemp(InPlanes))
		    , Vertices(MoveTemp(InVertices))
		{
			for (int32 ParticleIndex = 0; ParticleIndex < Vertices.Num(); ++ParticleIndex)
			{
				LocalBoundingBox.GrowToInclude(Vertices[ParticleIndex]);
			}

			// For now we approximate COM and volume with the bounding box
			CenterOfMass = LocalBoundingBox.GetCenterOfMass();
			Volume = LocalBoundingBox.GetVolume();

			ComputeUnitMassInertiaTensorAndRotationOfMass(Volume);
		}

		FConvex(TArray<FPlaneType>&& InPlanes, TArray<TArray<int32>>&& InFaceIndices, TArray<FVec3Type>&& InVertices)
		    : FImplicitObject(EImplicitObject::IsConvex | EImplicitObject::HasBoundingBox, ImplicitObjectType::Convex)
			, Planes(MoveTemp(InPlanes))
		    , Vertices(MoveTemp(InVertices))
		{
			for (int32 ParticleIndex = 0; ParticleIndex < Vertices.Num(); ++ParticleIndex)
			{
				LocalBoundingBox.GrowToInclude(Vertices[ParticleIndex]);
			}

			// For now we approximate COM and volume with the bounding box
			CenterOfMass = LocalBoundingBox.GetCenterOfMass();
			Volume = LocalBoundingBox.GetVolume();
			
			CreateStructureData(MoveTemp(InFaceIndices));

			ComputeUnitMassInertiaTensorAndRotationOfMass(Volume);
		}

		FConvex(TArray<FPlaneType>&& InPlanes, TArray<TArray<int32>>&& InFaceIndices, TArray<FVec3Type>&& InVertices, 
			const FVec3Type& InMin, const FVec3Type& InMax, const FRealType InVolume, const FVec3Type InInertiaTensor, const FRotation3& InRotationMatrix, const bool bRegularDatas)
			: FImplicitObject(EImplicitObject::IsConvex | EImplicitObject::HasBoundingBox, ImplicitObjectType::Convex)
			, Planes(MoveTemp(InPlanes))
			, Vertices(MoveTemp(InVertices))
		{
			LocalBoundingBox = FAABB3Type(InMin, InMax);
			CenterOfMass = LocalBoundingBox.GetCenterOfMass();
			
			Volume = InVolume;
			RotationOfMass = InRotationMatrix;
			UnitMassInertiaTensor = InInertiaTensor;
			
			CreateStructureData(MoveTemp(InFaceIndices), bRegularDatas);
		}

		FConvex(const TArray<FVec3Type>& InVertices, const FReal InMargin, FConvexBuilder::EBuildMethod BuildMethod = FConvexBuilder::EBuildMethod::Default)
		    : FImplicitObject(EImplicitObject::IsConvex | EImplicitObject::HasBoundingBox, ImplicitObjectType::Convex)
			, CenterOfMass(FVec3Type(0.f))
		{
			const int32 NumVertices = InVertices.Num();
			if (NumVertices == 0)
			{
				return;
			}

			TArray<TArray<int32>> FaceIndices;
			FConvexBuilder::Build(InVertices, Planes, FaceIndices, Vertices, LocalBoundingBox, BuildMethod);
			CHAOS_ENSURE(Planes.Num() == FaceIndices.Num());

			// @todo(chaos): this only works with triangles. Fix that an we can run MergeFaces before calling this
			CalculateVolumeAndCenterOfMass(Vertices, FaceIndices, Volume, CenterOfMass);

			if (BuildMethod != FConvexBuilder::EBuildMethod::ConvexHull3Simplified)
			{
				// it appears that this code path can leave the convex in an undefined state.
				// @todo(chaos): Tolerances should be based on size, or passed 
				const FRealType DistanceTolerance = 1.0f;
				const FRealType AngleTolerance = 1.e-6f;
				FConvexBuilder::MergeFaces(Planes, FaceIndices, Vertices, DistanceTolerance);
				FConvexBuilder::MergeColinearEdges(Planes, FaceIndices, Vertices, AngleTolerance);
				CHAOS_ENSURE(Planes.Num() == FaceIndices.Num());
			}

			CreateStructureData(MoveTemp(FaceIndices));

			ComputeUnitMassInertiaTensorAndRotationOfMass(Volume);

			SetMargin(InMargin);
		}

		FConvex& operator=(const FConvex& Other) = delete;
		
		FConvex& operator=(FConvex&& Other)
		{
			// Base class assignment
			// @todo(chaos): Base class needs protected assignment
			Type = Other.Type;
			CollisionType = Other.CollisionType;
			Margin = Other.Margin;
			bIsConvex = Other.bIsConvex;
			bDoCollide = Other.bDoCollide;
			bHasBoundingBox = Other.bHasBoundingBox;
#if TRACK_CHAOS_GEOMETRY
			bIsTracked = Other.bIsTracked;
#endif
			// This class assignment
			Planes = MoveTemp(Other.Planes);
			Vertices = MoveTemp(Other.Vertices);
			LocalBoundingBox = MoveTemp(Other.LocalBoundingBox);
			StructureData = MoveTemp(Other.StructureData);
			Volume = MoveTemp(Other.Volume);
			CenterOfMass = MoveTemp(Other.CenterOfMass);
			UnitMassInertiaTensor = MoveTemp(Other.UnitMassInertiaTensor);
			RotationOfMass = MoveTemp(Other.RotationOfMass);

			return *this;
		}

		Chaos::FConvex* RawCopyAsConvex() const
		{
			return new Chaos::FConvex(*this);
		}
		
		UE_DEPRECATED(5.4, "Please use RawCopyAsConvex instead")
		TUniquePtr<FConvex> CopyAsConvex() const
        {
			check(false);
            return nullptr;
        }

		CHAOS_API Chaos::FImplicitObjectPtr CopyGeometry() const override;
		CHAOS_API Chaos::FImplicitObjectPtr CopyGeometryWithScale(const FVec3& Scale) const override;
		CHAOS_API Chaos::FImplicitObjectPtr DeepCopyGeometry() const override;
		CHAOS_API Chaos::FImplicitObjectPtr DeepCopyGeometryWithScale(const FVec3& Scale) const override;

		CHAOS_API void MovePlanesAndRebuild(FRealType InDelta);

	private:
		CHAOS_API void CreateStructureData(TArray<TArray<int32>>&& FaceIndices, const bool bRegularDatas = false);

	public:
		static constexpr EImplicitObjectType StaticType()
		{
			return ImplicitObjectType::Convex;
		}

		virtual FReal GetMargin() const override
		{
			return Margin;
		}

		virtual FReal GetRadius() const override
		{
			return 0.0f;
		}

		virtual const FAABB3 BoundingBox() const override
		{
			// LWC : conversion from float to double
			return FAABB3(LocalBoundingBox.Min(), LocalBoundingBox.Max());
		}

		FConvex::FAABB3Type GetLocalBoundingBox() const 
		{
			return LocalBoundingBox;
		}

		// Return the distance to the surface
		virtual FReal PhiWithNormal(const FVec3& X, FVec3& Normal) const override
		{
			return PhiWithNormalInternal(X, Normal);
		}

		virtual FReal PhiWithNormalScaled(const FVec3& X, const FVec3& Scale, FVec3& Normal) const override
		{
			return PhiWithNormalScaledInternal(X, Scale, Normal);
		}

	private:
		// Distance to the surface
		FReal PhiWithNormalInternal(const FVec3& X, FVec3& Normal) const
		{
			const int32 NumPlanes = Planes.Num();
			if (NumPlanes == 0)
			{
				return FLT_MAX;
			}
			check(NumPlanes > 0);

			FReal MaxPhi = TNumericLimits<FReal>::Lowest();
			int32 MaxPlane = 0;

			for (int32 Idx = 0; Idx < NumPlanes; ++Idx)
			{
				const FReal Phi = Planes[Idx].SignedDistance(X);
				if (Phi > MaxPhi)
				{
					MaxPhi = Phi;
					MaxPlane = Idx;
				}
			}

			FReal Phi = Planes[MaxPlane].PhiWithNormal(X, Normal);
			if (Phi <= 0)
			{
				return Phi;
			}

			// If x is outside the convex mesh, we should find for the nearest point to triangles on the plane
			const int32 PlaneVerticesNum = NumPlaneVertices(MaxPlane);
			const FVec3 XOnPlane = X - Phi * Normal;
			FReal ClosestDistance = TNumericLimits<FReal>::Max();
			FVec3 ClosestPoint;
			for (int32 Index = 0; Index < PlaneVerticesNum - 2; Index++)
			{
				const FVec3 A(GetVertex(GetPlaneVertex(MaxPlane, 0)));
				const FVec3 B(GetVertex(GetPlaneVertex(MaxPlane, Index + 1)));
				const FVec3 C(GetVertex(GetPlaneVertex(MaxPlane, Index + 2)));

				const FVec3 Point = FindClosestPointOnTriangle(XOnPlane, A, B, C, X);
				if (XOnPlane == X)
				{
					return Phi;
				}

				const FReal Distance = (Point - XOnPlane).Size();
				if (Distance < ClosestDistance)
				{
					ClosestDistance = Distance;
					ClosestPoint = Point;
				}
			}

			const TVector<FReal, 3> Difference = X - ClosestPoint;
			Phi = Difference.Size();
			if (Phi > UE_SMALL_NUMBER)
			{
				Normal = (Difference) / Phi;
			}
			return Phi;
		}

		// Distance from a point to the surface for use in the scaled version. When the convex
		// is scaled, we need to bias the depth calculation to take into account the world-space scale
		FReal PhiWithNormalScaledInternal(const FVec3& X, const FVec3& Scale, FVec3& Normal) const
		{
			const int32 NumPlanes = Planes.Num();
			if (NumPlanes == 0)
			{
				return FLT_MAX;
			}
			check(NumPlanes > 0);

			FReal MaxPhi = TNumericLimits<FReal>::Lowest();
			FVec3 MaxNormal = FVec3(0,0,1);
			int32 MaxPlane = 0;
			for (int32 Idx = 0; Idx < NumPlanes; ++Idx)
			{
				FVec3 PlaneNormal = (Planes[Idx].Normal() / Scale).GetUnsafeNormal();
				FVec3 PlanePos = Planes[Idx].X() * Scale;
				FReal PlaneDistance = FVec3::DotProduct(X - PlanePos, PlaneNormal);
				if (PlaneDistance > MaxPhi)
				{
					MaxPhi = PlaneDistance;
					MaxNormal = PlaneNormal;
					MaxPlane = Idx;
				}
			}

			Normal = MaxNormal;

			if (MaxPhi < 0)
			{
				return MaxPhi;
			}

			// If X is outside the convex mesh, we should find for the nearest point to triangles on the plane
			const int32 PlaneVerticesNum = NumPlaneVertices(MaxPlane);
			const FVec3 XOnPlane = X - MaxPhi * Normal;
			FReal ClosestDistance = TNumericLimits<FReal>::Max();
			FVec3 ClosestPoint;
			for (int32 Index = 0; Index < PlaneVerticesNum - 2; Index++)
			{
				const FVec3 A(Scale * GetVertex(GetPlaneVertex(MaxPlane, 0)));
				const FVec3 B(Scale * GetVertex(GetPlaneVertex(MaxPlane, Index + 1)));
				const FVec3 C(Scale * GetVertex(GetPlaneVertex(MaxPlane, Index + 2)));

				const FVec3 Point = FindClosestPointOnTriangle(XOnPlane, A, B, C, X);
				if (XOnPlane == X)
				{
					return MaxPhi;
				}

				const FReal Distance = (Point - XOnPlane).Size();
				if (Distance < ClosestDistance)
				{
					ClosestDistance = Distance;
					ClosestPoint = Point;
				}
			}

			const FVec3 Difference = X - ClosestPoint;
			const FReal DifferenceLen = Difference.Size();
			if (DifferenceLen > UE_SMALL_NUMBER)
			{
				Normal = Difference / DifferenceLen;
				MaxPhi = DifferenceLen;
			}
			return MaxPhi;
		}


	public:
		/** Calls \c GJKRaycast(), which may return \c true but 0 for \c OutTime, 
		 * which means the bodies are touching, but not by enough to determine \c OutPosition 
		 * and \c OutNormal should be.  The burden for detecting this case is deferred to the
		 * caller. 
		 */
		CHAOS_API virtual bool Raycast(const FVec3& StartPoint, const FVec3& Dir, const FReal Length, const FReal Thickness, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex) const override;
		CHAOS_API bool RaycastFast(const FVec3& StartPoint, const FVec3& Dir, const FReal Length, const FReal Thickness, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex) const;

		virtual Pair<FVec3, bool> FindClosestIntersectionImp(const FVec3& StartPoint, const FVec3& EndPoint, const FReal Thickness) const override
		{
			const int32 NumPlanes = Planes.Num();
			TArray<Pair<FReal, FVec3>> Intersections;
			Intersections.Reserve(FMath::Min(static_cast<int32>(NumPlanes*.1), 16)); // Was NumPlanes, which seems excessive.
			for (int32 Idx = 0; Idx < NumPlanes; ++Idx)
			{ 
				// Offset plane with thickness in the normal direction and do a 0 thickness intersection query, so you only find intersections on the +Thickness*Normal plane, not the -Thickness*Normal plane.
				const Pair<FVec3, bool> PlaneIntersection = Thickness > 0 ? 
					FPlaneType(Planes[Idx].X() + Thickness * Planes[Idx].Normal(), Planes[Idx].Normal()).FindClosestIntersection(StartPoint, EndPoint, 0) :
					Planes[Idx].FindClosestIntersection(StartPoint, EndPoint, 0);
				if (PlaneIntersection.Second)
				{
					Intersections.Add(MakePair((FReal)(PlaneIntersection.First - StartPoint).SizeSquared(), PlaneIntersection.First));
				}
			}
			Intersections.Sort([](const Pair<FReal, FVec3>& Elem1, const Pair<FReal, FVec3>& Elem2) { return Elem1.First < Elem2.First; });
			for (const Pair<FReal, FVec3>& Elem : Intersections)
			{
				if (this->SignedDistance(Elem.Second) < (Thickness + 1e-4))
				{
					return MakePair(Elem.Second, true);
				}
			}
			return MakePair(FVec3(0), false);
		}

		// Whether the structure data has been created for this convex (will eventually always be true)
		bool HasStructureData() const { return StructureData.IsValid(); }

		// The convex structure data (mainly exposed for testing)
		const FConvexStructureData& GetStructureData() const { return StructureData; }

		// Get the index of the plane that most opposes the normal
		CHAOS_API int32 GetMostOpposingPlane(const FVec3& Normal) const;

		// Get the index of the plane that most opposes the normal
		CHAOS_API int32 GetMostOpposingPlaneScaled(const FVec3& Normal, const FVec3& Scale) const;

		// Get the nearest point on an edge and the edge vertices
		// Used for manifold generation
		CHAOS_API FVec3 GetClosestEdge(int32 PlaneIndexHint, const FVec3& Position, FVec3& OutEdgePos0, FVec3& OutEdgePos1) const;

		// Get the nearest point on an edge of the specified face
		FVec3 GetClosestEdgePosition(int32 PlaneIndex, const FVec3& Position) const
		{
			FVec3 Unused0, Unused1;
			return GetClosestEdge(PlaneIndex, Position, Unused0, Unused1);
		}

		CHAOS_API bool GetClosestEdgeVertices(int32 PlaneIndex, const FVec3& Position, int32& OutVertexIndex0, int32& OutVertexIndex1) const;

		// Get an array of all the plane indices that belong to a vertex (up to MaxVertexPlanes).
		// Returns the number of planes found.
		CHAOS_API int32 FindVertexPlanes(int32 VertexIndex, int32* OutVertexPlanes, int32 MaxVertexPlanes) const;

		CHAOS_API int32 GetVertexPlanes3(int32 VertexIndex, int32& PlaneIndex0, int32& PlaneIndex1, int32& PlaneIndex2) const;

		// The number of vertices that make up the corners of the specified face
		inline int32 NumPlaneVertices(int32 PlaneIndex) const
		{
			if (StructureData.IsValid())
			{
				return StructureData.NumPlaneVertices(PlaneIndex);
			}
			return 0;
		}

		// Get the vertex index of one of the vertices making up the corners of the specified face
		inline int32 GetPlaneVertex(int32 PlaneIndex, int32 PlaneVertexIndex) const
		{
			if (StructureData.IsValid())
			{
				return StructureData.GetPlaneVertex(PlaneIndex, PlaneVertexIndex);
			}
			return INDEX_NONE;
		}

		// Get the vertex index of one of the two vertices in an edge
		inline int32 GetEdgeVertex(int32 EdgeIndex, int32 EdgeVertexIndex) const
		{
			if (StructureData.IsValid())
			{
				return StructureData.GetEdgeVertex(EdgeIndex, EdgeVertexIndex);
			}
			return INDEX_NONE;
		}

		// Get the plane index of one of the two planes using an edge
		inline int32 GetEdgePlane(int32 EdgeIndex, int32 EdgePlaneIndex) const
		{
			if (StructureData.IsValid())
			{
				return StructureData.GetEdgePlane(EdgeIndex, EdgePlaneIndex);
			}
			return INDEX_NONE;
		}

		// Get the half edge index for a given plane
		inline int32 GetPlaneHalfEdge(int32 PlaneIndex, int32 PlaneEdgeIndex) const
		{
			if (StructureData.IsValid())
			{
				return StructureData.GetPlaneHalfEdge(PlaneIndex, PlaneEdgeIndex);
			}
			return INDEX_NONE;
		}

		// Get the half edges associated with an edge
		inline void GetHalfEdges(int32 EdgeIndex, int32& OutHalfEdge0, int32& OutHalfEdge1) const
		{
			if (StructureData.IsValid())
			{
				StructureData.GetHalfEdges(EdgeIndex, OutHalfEdge0, OutHalfEdge1);
				return;
			}

			OutHalfEdge0 = INDEX_NONE;
			OutHalfEdge1 = INDEX_NONE;
			return;
		}

		inline int32 NumPlanes() const
		{
			return Planes.Num();
		}

		inline int32 NumEdges() const
		{
			if (StructureData.IsValid())
			{
				return StructureData.NumEdges();
			}
			return 0;
		}

		inline int32 NumVertices() const
		{
			return (int32)Vertices.Num();
		}

		// Get the plane at the specified index (e.g., indices from FindVertexPlanes)
		inline const TPlaneConcrete<FReal, 3> GetPlane(int32 FaceIndex) const
		{
			// @todo(chaos) this is needed because this API is shared with BOx implicit - we shoudl eventually only need local space planes and be able to use single precision
			return TPlaneConcrete<FReal, 3>::MakeFrom(Planes[FaceIndex]);
		}

		inline void GetPlaneNX(const int32 FaceIndex, FVec3& OutN, FVec3& OutX) const
		{
			OutN = FVec3(Planes[FaceIndex].Normal());
			OutX = FVec3(Planes[FaceIndex].X());
		}

		inline const FPlaneType& GetPlaneRaw(int32 FaceIndex) const
		{
			return Planes[FaceIndex];
		}

		// Get the vertex at the specified index (e.g., indices from GetPlaneVertexs)
		inline const FVec3Type& GetVertex(int32 VertexIndex) const
		{
			return Vertices[VertexIndex];
		}


		CHAOS_API virtual int32 FindMostOpposingFace(const FVec3& Position, const FVec3& UnitDir, int32 HintFaceIndex, FReal SearchDist) const override;

		CHAOS_API virtual int32 FindMostOpposingFaceScaled(const FVec3& Position, const FVec3& UnitDir, int32 HintFaceIndex, FReal SearchDist, const FVec3& Scale) const override;

		FVec3 FindGeometryOpposingNormal(const FVec3& DenormDir, int32 FaceIndex, const FVec3& OriginalNormal) const
		{
			// For convexes, this function must be called with a face index.
			// If this ensure is getting hit, fix the caller so that it
			// passes in a valid face index.
			if (CHAOS_ENSURE(FaceIndex != INDEX_NONE))
			{
				const FPlaneType& OpposingFace = GetFaces()[FaceIndex];
				return OpposingFace.Normal();
			}
			return OriginalNormal;
		}

		CHAOS_API virtual int32 FindClosestFaceAndVertices(const FVec3& Position, TArray<FVec3>& FaceVertices, FReal SearchDist = 0.01f) const override;

		// Returns a winding order multiplier used in the manifold clipping and required when we have negative scales (See ImplicitObjectScaled)
		FReal GetWindingOrder() const
		{
			return 1.0f;
		}

	private:
		FORCEINLINE_DEBUGGABLE int32 GetSupportVertex(const FVec3Type& Direction) const
		{
			FRealType MaxDot = TNumericLimits<FRealType>::Lowest();
			int32 MaxVIdx = INDEX_NONE;
			const int32 NumVertices = Vertices.Num();

			for (int32 Idx = 0; Idx < NumVertices; ++Idx)
			{
				const FRealType Dot = FVec3Type::DotProduct(Vertices[Idx], Direction);
				if (Dot > MaxDot)
				{
					MaxDot = Dot;
					MaxVIdx = Idx;
				}
			}

			return MaxVIdx;
		}

	public:

		// @todo(chaos): Move to utils
		inline bool IntersectPlanes3(const FVec3& X1, const FVec3& N1, const FVec3& X2, const FVec3& N2, const FVec3& X3, const FVec3& N3, FVec3& OutX, const FReal EpsilonSq = FReal(1.e-6)) const
		{
			// Compute determinant, the triple product P1|(P2^P3)==(P1^P2)|P3.
			const FVec3 N1CrossN2 = FVec3::CrossProduct(N1, N2);
			const FReal Det = FVec3::DotProduct(N1CrossN2, N3);
			if (FMath::Square(Det) < EpsilonSq)
			{
				// Degenerate.
				OutX = FVec3(0);
				return false;
			}
			else
			{
				// Compute the intersection point, guaranteed valid if determinant is nonzero.
				const FVec3 N2CrossN3 = FVec3::CrossProduct(N2, N3);
				const FVec3 N3CrossN1 = FVec3::CrossProduct(N3, N1);
				const FReal D1 = FVec3::DotProduct(X1, N1);
				const FReal D2 = FVec3::DotProduct(X2, N2);
				const FReal D3 = FVec3::DotProduct(X3, N3);
				OutX = (D1 * N2CrossN3 + D2 * N3CrossN1 + D3 * N1CrossN2) / Det;
			}
			return true;
		}

		FVec3 GetMarginAdjustedVertex(const int32 VertexIndex, const FReal InMargin, FReal* OutSupportDelta) const
		{
			// @chaos(todo): moving the vertices this way based on margin is only valid for small margins. If the margin
			// is large enough to cause a face to reduce to zero size, vertices should be merged and the path is non-linear.
			// This can be fixed with some extra data in the convex structure, but for now we accept the fact that large 
			// margins on convexes with small faces can cause non-convex core shapes.

			if (InMargin == FReal(0))
			{
				return GetVertex(VertexIndex);
			}

			// Get any 3 planes that contribute to this vertex
			int32 PlaneIndex0 = INDEX_NONE;
			int32 PlaneIndex1 = INDEX_NONE;
			int32 PlaneIndex2 = INDEX_NONE;
			const int32 NumVertexPlanes = GetVertexPlanes3(VertexIndex, PlaneIndex0, PlaneIndex1, PlaneIndex2);

			// Move the planes by the margin and recalculate the interection
			if (NumVertexPlanes >= 3)
			{
				FVec3 VertexPos = Vertices[VertexIndex];
				if (IntersectPlanes3(
					VertexPos - InMargin * Planes[PlaneIndex0].Normal(), Planes[PlaneIndex0].Normal(),
					VertexPos - InMargin * Planes[PlaneIndex1].Normal(), Planes[PlaneIndex1].Normal(),
					VertexPos - InMargin * Planes[PlaneIndex2].Normal(), Planes[PlaneIndex2].Normal(),
					VertexPos))
				{
					if (OutSupportDelta != nullptr)
					{
						*OutSupportDelta = (Vertices[VertexIndex] - VertexPos).Size() - InMargin;
					}
					return VertexPos;
				}
			}

			// If we get here, the convex hull is malformed. Try to handle it anyway 
			// @todo(chaos): track down the invalid hull issue

			if (NumVertexPlanes == 2)
			{
				const FVec3 NewPlaneX = GetVertex(VertexIndex);
				const FVec3 NewPlaneN0 = Planes[PlaneIndex0].Normal();
				const FVec3 NewPlaneN1 = Planes[PlaneIndex1].Normal();
				const FVec3 NewPlaneN = (NewPlaneN0 + NewPlaneN1).GetSafeNormal();
				return NewPlaneX - (InMargin * NewPlaneN);
			}

			if (NumVertexPlanes == 1)
			{
				const FVec3 NewPlaneX = GetVertex(VertexIndex);
				const FVec3 NewPlaneN = Planes[PlaneIndex0].Normal();
				return NewPlaneX - (InMargin * NewPlaneN);
			}

			// Ok now we really are done...just return the outer vertex and duck
			return GetVertex(VertexIndex);
		}

		FVec3 GetMarginAdjustedVertexScaled(int32 VertexIndex, FReal InMargin, const FVec3& Scale, FReal* OutSupportDelta) const
		{
			// Get any 3 planes that contribute to this vertex
			int32 PlaneIndex0 = INDEX_NONE;
			int32 PlaneIndex1 = INDEX_NONE;
			int32 PlaneIndex2 = INDEX_NONE;
			const int32 NumVertexPlanes = GetVertexPlanes3(VertexIndex, PlaneIndex0, PlaneIndex1, PlaneIndex2);
			const FVec3 InvScale = FVec3(FReal(1) / Scale.X, FReal(1) / Scale.Y, FReal(1) / Scale.Z);

			// Move the planes by the margin and recalculate the interection
			if (NumVertexPlanes >= 3)
			{
				const FVec3 VertexPos = Scale * Vertices[VertexIndex];

				const FVec3 NewPlaneN0 = (Planes[PlaneIndex0].Normal() * InvScale).GetUnsafeNormal();
				const FVec3 NewPlaneN1 = (Planes[PlaneIndex1].Normal() * InvScale).GetUnsafeNormal();
				const FVec3 NewPlaneN2 = (Planes[PlaneIndex2].Normal() * InvScale).GetUnsafeNormal();

				FVec3 AdjustedVertexPos = VertexPos;
				if (IntersectPlanes3(
					VertexPos - InMargin * NewPlaneN0, NewPlaneN0,
					VertexPos - InMargin * NewPlaneN1, NewPlaneN1,
					VertexPos - InMargin * NewPlaneN2, NewPlaneN2,
					AdjustedVertexPos))
				{
					if (OutSupportDelta != nullptr)
					{
						*OutSupportDelta = (VertexPos - AdjustedVertexPos).Size() - InMargin;
					}
					return AdjustedVertexPos;
				}
			}

			// If we get here, the convex hull is malformed. Try to handle it anyway 
			// @todo(chaos): track down the invalid hull issue

			if (NumVertexPlanes == 2)
			{
				const FVec3 NewPlaneX = Scale * GetVertex(VertexIndex);
				const FVec3 NewPlaneN0 = (Planes[PlaneIndex0].Normal() * InvScale).GetUnsafeNormal();
				const FVec3 NewPlaneN1 = (Planes[PlaneIndex1].Normal() * InvScale).GetUnsafeNormal();
				const FVec3 NewPlaneN = (NewPlaneN0 + NewPlaneN1).GetSafeNormal();
				return NewPlaneX - (InMargin * NewPlaneN);
			}

			if (NumVertexPlanes == 1)
			{
				const FVec3 NewPlaneX = Scale * GetVertex(VertexIndex);
				const FVec3 NewPlaneN = (Planes[PlaneIndex0].Normal() * InvScale).GetUnsafeNormal();
				return NewPlaneX - (InMargin * NewPlaneN);
			}

			// Ok now we really are done...just return the outer vertex and duck
			return GetVertex(VertexIndex) * Scale;
		}

	public:
		// Return support point on the core shape (the convex shape with all planes moved inwards by margin).
		FVec3 SupportCore(const FVec3& Direction, const FReal InMargin, FReal* OutSupportDelta, int32& VertexIndex) const
		{
			const int32 SupportVertexIndex = GetSupportVertex(FVec3Type(Direction));
			VertexIndex = SupportVertexIndex;
			if (SupportVertexIndex != INDEX_NONE)
			{
				return GetMarginAdjustedVertex(SupportVertexIndex, InMargin, OutSupportDelta);
			}
			return FVec3(0);
		}

		// Return support point on the core shape (the convex shape with all planes moved inwards by margin).
		VectorRegister4Float SupportCoreSimd(const VectorRegister4Float& Direction, const FReal InMargin) const
		{
			FVec3 DirectionVec3;
			VectorStoreFloat3(Direction, &DirectionVec3);
			const int32 SupportVertexIndex = GetSupportVertex(DirectionVec3);
			if (SupportVertexIndex != INDEX_NONE)
			{
				FVec3 SupportVert =  GetMarginAdjustedVertex(SupportVertexIndex, InMargin, nullptr);
				return MakeVectorRegisterFloatFromDouble(MakeVectorRegister(SupportVert.X, SupportVert.Y, SupportVert.Z, 0.0));
			}
			return VectorZeroFloat();
		}


		// SupportCore with non-uniform scale support. This is required for the margin in scaled
		// space to by uniform. Note in this version all the inputs are in outer container's (scaled shape) space
		FVec3 SupportCoreScaled(const FVec3& Direction, FReal InMargin, const FVec3& Scale, FReal* OutSupportDelta, int32& VertexIndex) const
		{
			// Find the supporting vertex index
			const FVec3Type DirectionScaled = FVec3Type(Scale * Direction);	// does not need to be normalized
			const int32 SupportVertexIndex = GetSupportVertex(DirectionScaled);
			VertexIndex = SupportVertexIndex;
			// Adjust the vertex position based on margin
			if (SupportVertexIndex != INDEX_NONE)
			{
				if (InMargin == 0.0f)
				{
					return FVec3(GetVertex(VertexIndex)) * Scale;
				}
				else
				{
					// Note: Shapes wrapped in a non-uniform scale should not have their own margin and we assume that here
					// @chaos(todo): apply an upper limit to the margin to prevent a non-convex or null shape (also see comments in GetMarginAdjustedVertex)
					return FVec3(GetMarginAdjustedVertexScaled(SupportVertexIndex, InMargin, Scale, OutSupportDelta));
				}
			}
			return FVec3(0);
		}

		// Return support point on the shape
		// @todo(chaos): do we need to support thickness?
		FORCEINLINE FVec3 Support(const FVec3& Direction, const FReal Thickness, int32& VertexIndex) const
		{
			const int32 MaxVIdx = GetSupportVertex(FVec3Type(Direction));
			VertexIndex = MaxVIdx;
			if (MaxVIdx != INDEX_NONE)
			{
				if (Thickness != 0.0f)
				{
					return Vertices[MaxVIdx] + Direction.GetUnsafeNormal() * Thickness;
				}
				return Vertices[MaxVIdx];
			}
			return FVec3(0);
		}

		FORCEINLINE FVec3 SupportScaled(const FVec3& Direction, const FReal Thickness, const FVec3& Scale, int32& VertexIndex) const
		{
			FVec3 SupportPoint = Support(Direction * Scale, 0.0f, VertexIndex) * Scale;
			if (Thickness > 0.0f)
			{
				SupportPoint += Thickness * Direction.GetSafeNormal();
			}
			return SupportPoint;
		}

		virtual FString ToString() const override
		{
			return ToStringSummary();
		}

		// A one-line summary of the convex
		FString ToStringSummary() const
		{
			return FString::Printf(TEXT("Convex: Verts: %d, Edges %d, Planes: %d, Margin %f"), NumVertices(), NumEdges(), NumPlanes(), GetMargin());
		}

		// Print all the datas
		FString ToStringFull() const
		{
			FString S = ToStringSummary();
			S.Append(TEXT("\n"));
			for (int32 VertexIndex = 0; VertexIndex < NumVertices(); ++VertexIndex)
			{
				const FVec3 Vert = GetVertex(VertexIndex);
				S.Append(FString::Printf(TEXT("  Vertex %d: [%f, %f, %f]\n"), VertexIndex, Vert.X, Vert.Y, Vert.Z));
			}
			for (int32 PlaneIndex = 0; PlaneIndex < NumPlanes(); ++PlaneIndex)
			{
				const TPlaneConcrete<FReal, 3> Plane = GetPlane(PlaneIndex);
				S.Append(FString::Printf(TEXT("  Plane %d: Normal: [%f, %f, %f], Distance: %f, Verts: ["), PlaneIndex, Plane.Normal().X, Plane.Normal().Y, Plane.Normal().Z, FVec3::DotProduct(Plane.X(), Plane.Normal())));
				const int32 PlaneVertexCount = NumPlaneVertices(PlaneIndex);
				for (int32 PlaneVertexIndex = 0; PlaneVertexIndex < PlaneVertexCount; ++PlaneVertexIndex)
				{
					S.Append(FString::Printf(TEXT("%d"), GetPlaneVertex(PlaneIndex, PlaneVertexIndex)));
					if (PlaneVertexIndex < PlaneVertexCount - 1)
					{
						S.Append(FString::Printf(TEXT(", ")));
					}
				}
				S.Append(FString::Printf(TEXT("]\n")));
			}
			S.Append(FString::Printf(TEXT("  Edges: ")));
			for (int32 EdgeIndex = 0; EdgeIndex < NumEdges(); ++EdgeIndex)
			{
				S.Append(FString::Printf(TEXT("[%d, %d]"), GetEdgeVertex(EdgeIndex, 0), GetEdgeVertex(EdgeIndex, 1)));
				if (EdgeIndex < NumEdges() - 1)
				{
					S.Append(FString::Printf(TEXT(", ")));
				}
			}
			S.Append(FString::Printf(TEXT("\n")));
			return S;
		}

		const TArray<FVec3Type>& GetVertices() const
		{
			return Vertices;
		}

		const TArray<FPlaneType>& GetFaces() const
		{
			return Planes;
		}

		const FReal GetVolume() const
		{
			return Volume;
		}
		
		const FMatrix33 GetInertiaTensor(const FReal Mass) const
		{
			const FVec3 ScaledInertiaTensorDiagonal{ UnitMassInertiaTensor * Mass };
			return FMatrix33(ScaledInertiaTensorDiagonal.X, ScaledInertiaTensorDiagonal.Y, ScaledInertiaTensorDiagonal.Z);
		}

		FRotation3 GetRotationOfMass() const
		{
			return RotationOfMass;
		}

		const FVec3 GetCenterOfMass() const
		{
			return CenterOfMass;
		}

		virtual uint32 GetTypeHash() const override
		{
			uint32 Result = LocalBoundingBox.GetTypeHash();

			for (const FVec3Type& Vertex: Vertices)
			{
				Result = HashCombine(Result, ::GetTypeHash(Vertex[0]));
				Result = HashCombine(Result, ::GetTypeHash(Vertex[1]));
				Result = HashCombine(Result, ::GetTypeHash(Vertex[2]));
			}

			for(const FPlaneType& Plane : Planes)
			{
				Result = HashCombine(Result, Plane.GetTypeHash());
			}

			return Result;
		}

		FORCEINLINE void SerializeImp(FArchive& Ar)	
		{
			Ar.UsingCustomVersion(FExternalPhysicsCustomObjectVersion::GUID);
			Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
			Ar.UsingCustomVersion(FPhysicsObjectVersion::GUID);
			Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
			Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);
			FImplicitObject::SerializeImp(Ar);

			if (Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) < FExternalPhysicsCustomObjectVersion::ConvexUsesTPlaneConcrete)
			{
				TArray<TPlane<FRealType, 3>> TmpPlanes;
				Ar << TmpPlanes;

				Planes.SetNum(TmpPlanes.Num());
				for(int32 Idx = 0; Idx < Planes.Num(); ++Idx)
				{
					Planes[Idx] = TmpPlanes[Idx].PlaneConcrete();
				}
			}
			else
			{
				Ar << Planes;
			}

			// Do we use the old Particles array or the new Vertices array?
			// Note: This change was back-ported to UE4, so we need to check 
			// multiple object versions.
			// This is a mess because the change was back-integrated to 2 different streams. Be careful...
			bool bConvexVerticesNewFormatUE4 = (Ar.CustomVer(FPhysicsObjectVersion::GUID) >= FPhysicsObjectVersion::ConvexUsesVerticesArray);
			bool bConvexVerticesNewFormatUE5 = (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) >= FUE5MainStreamObjectVersion::ConvexUsesVerticesArray);
			bool bConvexVerticesNewFormatFN = (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::ChaosConvexVariableStructureDataAndVerticesArray);
			bool bConvexVerticesNewFormat = bConvexVerticesNewFormatUE4 || bConvexVerticesNewFormatUE5 || bConvexVerticesNewFormatFN;

			if (!bConvexVerticesNewFormat)
			{
				TParticles<FRealType, 3> TmpSurfaceParticles;
				Ar << TmpSurfaceParticles;

				const int32 NumVertices = (int32)TmpSurfaceParticles.Size();
				Vertices.SetNum(NumVertices);
				for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
				{
					Vertices[VertexIndex] = TmpSurfaceParticles.GetX(VertexIndex);
				}
			}
			else
			{
				Ar << Vertices;
			}
			

			TBox<FRealType,3>::SerializeAsAABB(Ar, LocalBoundingBox);

			if (Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) >= FExternalPhysicsCustomObjectVersion::AddConvexCenterOfMassAndVolume)
			{
				FRealSingle VolumeFloat = (FRealSingle)Volume; // LWC_TODO : potential precision lossdepending on FRealType, to be changed when we can serialize FReal as double
				Ar << VolumeFloat;
				Volume = (FRealType)VolumeFloat;

				// Some assets have stored NaN ( empty convex objects from Geometry Collection )
				// Fixing this so that next time they get saved the Volume is a valid number
				if (FMath::IsNaN(Volume) && Ar.IsLoading())
				{
					Volume = 0;
				}

				Ar << CenterOfMass;
			}
			else if (Ar.IsLoading())
			{
				// Rebuild convex in order to extract face indices.
				// @todo(chaos): Make it so it can take Vertices as both input and output without breaking...
				TArray<TArray<int32>> FaceIndices;
				TArray<FVec3Type> TempVertices;
				FConvexBuilder::Build(Vertices, Planes, FaceIndices, TempVertices, LocalBoundingBox);

				// Copy vertices and move into particles.
				CalculateVolumeAndCenterOfMass(Vertices, FaceIndices, Volume, CenterOfMass);
			}

			Ar.UsingCustomVersion(FReleaseObjectVersion::GUID);
			if (Ar.CustomVer(FReleaseObjectVersion::GUID) >= FReleaseObjectVersion::MarginAddedToConvexAndBox)
			{
				FRealSingle MarginFloat = (FRealSingle)FImplicitObject::Margin;
				Ar << MarginFloat;
				FImplicitObject::Margin = MarginFloat;
			}

			if (Ar.CustomVer(FReleaseObjectVersion::GUID) >= FReleaseObjectVersion::StructureDataAddedToConvex)
			{
				Ar << StructureData;
			}
			else if (Ar.IsLoading())
			{
				// Generate the structure data from the planes and vertices
				TArray<TArray<int32>> FaceIndices;
				FConvexBuilder::BuildPlaneVertexIndices(Planes, Vertices, FaceIndices);
				CreateStructureData(MoveTemp(FaceIndices));
			}

			if (Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) >= FUE5ReleaseStreamObjectVersion::AddedInertiaTensorAndRotationOfMassAddedToConvex)
			{
				Ar << UnitMassInertiaTensor;
				Ar << RotationOfMass;
			}
			else if (Ar.IsLoading())
			{
				ComputeUnitMassInertiaTensorAndRotationOfMass(Volume);
			}
		}

		virtual void Serialize(FChaosArchive& Ar) override
		{
			FChaosArchiveScopedMemory ScopedMemory(Ar, GetTypeName());
			SerializeImp(Ar);
		}

		virtual void Serialize(FArchive& Ar) override
		{
			SerializeImp(Ar);
		}

		virtual bool IsValidGeometry() const override
		{
			return (Vertices.Num() > 0 && Planes.Num() > 0);
		}

		virtual bool IsPerformanceWarning() const override
		{
			return FConvexBuilder::IsPerformanceWarning(Planes.Num(), Vertices.Num());
		}

		virtual FString PerformanceWarningAndSimplifaction() override
		{

			FString PerformanceWarningString = FConvexBuilder::PerformanceWarningString(Planes.Num(), Vertices.Num());
			if (FConvexBuilder::IsGeometryReductionEnabled())
			{
				PerformanceWarningString += ", [Simplifying]";
				SimplifyGeometry();
			}

			return PerformanceWarningString;
		}

		void SimplifyGeometry()
		{
			TArray<TArray<int32>> FaceIndices;
			FConvexBuilder::Simplify(Planes, FaceIndices, Vertices, LocalBoundingBox);

			// @todo(chaos): Tolerances should be based on size, or passed in
			const FRealType DistanceTolerance = 1.0f;
			const FRealType AngleTolerance = 1.e-6f;
			FConvexBuilder::MergeFaces(Planes, FaceIndices, Vertices, DistanceTolerance);
			FConvexBuilder::MergeColinearEdges(Planes, FaceIndices, Vertices, AngleTolerance);

			CreateStructureData(MoveTemp(FaceIndices));
		}

		FVec3 GetCenter() const
		{
			return GetCenterOfMass();
		}

#if INTEL_ISPC
		// See PerParticlePBDCollisionConstraint.cpp
		// ISPC code has matching structs for interpreting FImplicitObjects.
		// This is used to verify that the structs stay the same.
		struct FISPCDataVerifier
		{
			static constexpr int32 OffsetOfPlanes() { return offsetof(FConvex, Planes); }
			static constexpr int32 SizeOfPlanes() { return sizeof(FConvex::Planes); }
			static constexpr int32 OffsetOfVertices() { return offsetof(FConvex, Vertices); }
			static constexpr int32 SizeOfVertices() { return sizeof(FConvex::Vertices); }
			static constexpr int32 OffsetOfStructureData() { return offsetof(FConvex, StructureData); }
			static constexpr int32 SizeOfStructureData() { return sizeof(FConvex::StructureData); }
		};
		friend FISPCDataVerifier;
#endif // #if INTEL_ISPC

	private:
		CHAOS_API void ComputeUnitMassInertiaTensorAndRotationOfMass(const FReal InVolume);
		
		// IMPORTANT to keep this copy constructor private to avoid unintented expensive copies 
		FConvex(const FConvex& Other)
			: FImplicitObject(EImplicitObject::IsConvex | EImplicitObject::HasBoundingBox, ImplicitObjectType::Convex)
			, Planes(Other.Planes)
			, Vertices(Other.Vertices)
			, LocalBoundingBox(Other.LocalBoundingBox)
			, Volume(Other.Volume)
			, CenterOfMass(Other.CenterOfMass)
			, UnitMassInertiaTensor(Other.UnitMassInertiaTensor)
			, RotationOfMass(Other.RotationOfMass)
		{
			StructureData.CopyFrom(Other.StructureData);
			SetMargin(Other.Margin);
		}

	private:
		TArray<FPlaneType> Planes;
		TArray<FVec3Type> Vertices; //copy of the vertices that are just on the convex hull boundary
		FAABB3Type LocalBoundingBox;
		FConvexStructureData StructureData;
		FRealType Volume;
		FVec3Type CenterOfMass;
		FVec3 UnitMassInertiaTensor;
		FRotation3 RotationOfMass;
	};
}
