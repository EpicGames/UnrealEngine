// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/ConvexHalfEdgeStructureData.h"
#include "ChaosArchive.h"
#include "ChaosCheck.h"
#include "ChaosLog.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/PhysicsObjectVersion.h"

//
//
// LEGACY CODE
// Kept around for serialization of old data. Most of the API has been removed, 
// except what is needed to convert to the new format. 
// See ConvexHalfEdgeStructureData.h for the replacement.
//
//

namespace Chaos
{
	namespace Legacy
	{
		class FLegacyConvexStructureDataLoader;

		// Base class for TConvexFlattenedArrayStructureData.
		// NOTE: Deliberately no-virtual destructor so it should never be deleted from a 
		// base class pointer.
		class FConvexFlattenedArrayStructureData
		{
		public:
			FConvexFlattenedArrayStructureData() {}
			~FConvexFlattenedArrayStructureData() = default;
		};

		// A container for the convex structure data arrays.
		// This is templated on the index type required, which needs to
		// be large enough to hold the max of the number of vertices or 
		// planes on the convex.
		// 
		// T_INDEX:			the type used to index into the convex vertices 
		//					and planes array (in the outer convex). Must be 
		//					able to contain max(NumPlanes, NumVerts).
		//
		// T_OFFSETINDEX:	the type used to index the flattened array of
		//					indices. Must be able to contain 
		//					Max(NumPlanes*AverageVertsPerPlane)
		//
		template<typename T_INDEX, typename T_OFFSETINDEX>
		class TConvexFlattenedArrayStructureData : public FConvexFlattenedArrayStructureData
		{
		public:
			// Everything public - not used outside of legacy loader and unit tests

			using FIndex = T_INDEX;
			using FOffsetIndex = T_OFFSETINDEX;

			FORCEINLINE int32 NumVertices() const
			{
				return VertexPlanesOffsetCount.Num();
			}

			int32 NumPlanes() const
			{
				return PlaneVerticesOffsetCount.Num();
			}

			// The number of planes that use the specified vertex
			int32 NumVertexPlanes(int32 VertexIndex) const
			{
				return VertexPlanesOffsetCount[VertexIndex].Value;
			}

			// Get the plane index (in the outer convex container) of one of the planes that uses the specified vertex
			int32 GetVertexPlane(int32 VertexIndex, int32 VertexPlaneIndex) const
			{
				check(VertexPlaneIndex < NumVertexPlanes(VertexIndex));

				const int32 VertexPlaneFlatArrayIndex = VertexPlanesOffsetCount[VertexIndex].Key + VertexPlaneIndex;
				return (int32)VertexPlanes[VertexPlaneFlatArrayIndex];
			}

			// The number of vertices that make up the corners of the specified face
			int32 NumPlaneVertices(int32 PlaneIndex) const
			{
				return PlaneVerticesOffsetCount[PlaneIndex].Value;
			}

			// Get the vertex index (in the outer convex container) of one of the vertices making up the corners of the specified face
			int32 GetPlaneVertex(int32 PlaneIndex, int32 PlaneVertexIndex) const
			{
				check(PlaneVertexIndex < PlaneVerticesOffsetCount[PlaneIndex].Value);

				const int32 PlaneVertexFlatArrayIndex = PlaneVerticesOffsetCount[PlaneIndex].Key + PlaneVertexIndex;
				return (int32)PlaneVertices[PlaneVertexFlatArrayIndex];
			}

			void SetPlaneVertices(const TArray<TArray<int32>>& InPlaneVertices, int32 NumVerts)
			{
				// We flatten the ragged arrays into a single array, and store a seperate
				// arrray of [index,count] tuples to reproduce the ragged arrays.
				Reset();

				// Generate the ragged array [offset,count] tuples
				PlaneVerticesOffsetCount.SetNumZeroed(InPlaneVertices.Num());
				VertexPlanesOffsetCount.SetNumZeroed(NumVerts);

				// Count the number of planes for each vertex (store it in the tuple)
				int FlatArrayIndexCount = 0;
				for (int32 PlaneIndex = 0; PlaneIndex < InPlaneVertices.Num(); ++PlaneIndex)
				{
					FlatArrayIndexCount += InPlaneVertices[PlaneIndex].Num();

					for (int32 PlaneVertexIndex = 0; PlaneVertexIndex < InPlaneVertices[PlaneIndex].Num(); ++PlaneVertexIndex)
					{
						const int32 VertexIndex = InPlaneVertices[PlaneIndex][PlaneVertexIndex];
						VertexPlanesOffsetCount[VertexIndex].Value++;
					}
				}

				// Initialize the flattened arrary offsets and reset the count (we re-increment it when copying the data below)
				int VertexPlanesArrayStart = 0;
				for (int32 VertexIndex = 0; VertexIndex < NumVerts; ++VertexIndex)
				{
					VertexPlanesOffsetCount[VertexIndex].Key = VertexPlanesArrayStart;
					VertexPlanesArrayStart += VertexPlanesOffsetCount[VertexIndex].Value;
					VertexPlanesOffsetCount[VertexIndex].Value = 0;
				}

				// Allocate space for the flattened arrays
				PlaneVertices.SetNumZeroed(FlatArrayIndexCount);
				VertexPlanes.SetNumZeroed(FlatArrayIndexCount);

				// Copy the indices into the flattened arrays
				int32 PlaneVerticesArrayStart = 0;
				for (int32 PlaneIndex = 0; PlaneIndex < InPlaneVertices.Num(); ++PlaneIndex)
				{
					PlaneVerticesOffsetCount[PlaneIndex].Key = PlaneVerticesArrayStart;
					PlaneVerticesOffsetCount[PlaneIndex].Value = InPlaneVertices[PlaneIndex].Num();
					PlaneVerticesArrayStart += InPlaneVertices[PlaneIndex].Num();

					for (int32 PlaneVertexIndex = 0; PlaneVertexIndex < InPlaneVertices[PlaneIndex].Num(); ++PlaneVertexIndex)
					{
						const int32 VertexIndex = InPlaneVertices[PlaneIndex][PlaneVertexIndex];

						const int32 PlaneVertexFlatArrayIndex = PlaneVerticesOffsetCount[PlaneIndex].Key + PlaneVertexIndex;
						PlaneVertices[PlaneVertexFlatArrayIndex] = VertexIndex;

						const int32 VertexPlaneFlatArrayIndex = VertexPlanesOffsetCount[VertexIndex].Key + VertexPlanesOffsetCount[VertexIndex].Value;
						VertexPlanesOffsetCount[VertexIndex].Value++;
						VertexPlanes[VertexPlaneFlatArrayIndex] = PlaneIndex;
					}
				}
			}

			void Reset()
			{
				PlaneVerticesOffsetCount.Reset();
				VertexPlanesOffsetCount.Reset();
				PlaneVertices.Reset();
				VertexPlanes.Reset();
			}

			void Serialize(FArchive& Ar)
			{
				Ar << PlaneVerticesOffsetCount;
				Ar << VertexPlanesOffsetCount;
				Ar << PlaneVertices;
				Ar << VertexPlanes;
			}

			friend FArchive& operator<<(FArchive& Ar, TConvexFlattenedArrayStructureData<T_INDEX, T_OFFSETINDEX>& Value)
			{
				Value.Serialize(Ar);
				return Ar;
			}


			// Array of [offset, count] for each plane that gives the set of indices in the PlaneVertices flattened array
			TArray<TPair<FOffsetIndex, FIndex>> PlaneVerticesOffsetCount;

			// Array of [offset, count] for each vertex that gives the set of indices in the VertexPlanes flattened array
			TArray<TPair<FOffsetIndex, FIndex>> VertexPlanesOffsetCount;

			// A flattened ragged array. For each plane: the set of vertex indices that form the corners of the face in counter-clockwise order
			TArray<FIndex> PlaneVertices;

			// A flattened ragged array. For each vertex: the set of plane indices that use the vertex
			TArray<FIndex> VertexPlanes;
		};

		using FConvexFlattenedArrayStructureDataS32 = TConvexFlattenedArrayStructureData<int32, int32>;
		using FConvexFlattenedArrayStructureDataU8 = TConvexFlattenedArrayStructureData<uint8, uint16>;

		//
		// A utility for loading old convex structure data
		//
		class FLegacyConvexStructureDataLoader
		{
		public:
			// Load data from an asset saved before we had a proper half-edge data structure
			static void Load(FArchive& Ar, TArray<TArray<int32>>& OutPlaneVertices, int32& OutNumVertices)
			{
				bool bUseVariableSizeStructureDataUE4 = Ar.CustomVer(FPhysicsObjectVersion::GUID) >= FPhysicsObjectVersion::VariableConvexStructureData;
				bool bUseVariableSizeStructureDataFN = Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::ChaosConvexVariableStructureDataAndVerticesArray;
				bool bUseVariableSizeStructureData = bUseVariableSizeStructureDataUE4 || bUseVariableSizeStructureDataFN;

				if (!bUseVariableSizeStructureData)
				{
					LoadFixedSizeRaggedArrays(Ar, OutPlaneVertices, OutNumVertices);
				}
				else
				{
					LoadVariableSizePackedArrays(Ar, OutPlaneVertices, OutNumVertices);
				}
			}

		private:
			enum class EIndexType : int8
			{
				None,
				S32,
				U8,
			};

			// Orginal structure used 32bit indices regardless of max index, and they were store in ragged TArray<TArray<int32>> of vertex indices per plane
			static void LoadFixedSizeRaggedArrays(FArchive& Ar, TArray<TArray<int32>>& OutPlaneVertices, int32& OutNumVertices)
			{
				TArray<TArray<int32>> OldPlaneVertices;
				TArray<TArray<int32>> OldVertexPlanes;
				Ar << OldPlaneVertices;
				Ar << OldVertexPlanes;

				OutPlaneVertices = MoveTemp(OldPlaneVertices);
				OutNumVertices = OldVertexPlanes.Num();
			}

			// The second data structure packed the indices into flat arrays, and used an 8bit index if possible (and 32bit otherwise)
			static void LoadVariableSizePackedArrays(FArchive& Ar, TArray<TArray<int32>>& OutPlaneVertices, int32& OutNumVertices)
			{
				EIndexType OldIndexType;
				TArray<TArray<int32>> OldPlaneVertices;
				int32 OldNumVertices = 0;

				Ar << OldIndexType;

				// Read the old structure and pull out the data we need to generate the new one (the output params)
				if (OldIndexType == EIndexType::S32)
				{
					FConvexFlattenedArrayStructureDataS32 OldData;
					Ar << OldData;
					ExtractData(OldData, OldPlaneVertices, OldNumVertices);
				}
				else if (OldIndexType == EIndexType::U8)
				{
					FConvexFlattenedArrayStructureDataU8 OldData;
					Ar << OldData;
					ExtractData(OldData, OldPlaneVertices, OldNumVertices);
				}

				OutPlaneVertices = MoveTemp(OldPlaneVertices);
				OutNumVertices = OldNumVertices;
			}

			// Extract ragged arrays of vertices per plane from the flattened array structure
			// For use by LoadVariableSizePackedArrays to avoid code dupe based on index size
			template<typename T_OLDCONTAINER>
			static void ExtractData(const T_OLDCONTAINER& OldData, TArray<TArray<int32>>& OldPlaneVertices, int32& OldNumVertices)
			{
				OldPlaneVertices.SetNum(OldData.NumPlanes());
				for (int32 PlaneIndex = 0; PlaneIndex < OldData.NumPlanes(); ++PlaneIndex)
				{
					OldPlaneVertices[PlaneIndex].SetNum(OldData.NumPlaneVertices(PlaneIndex));
					for (int32 PlaneVertexIndex = 0; PlaneVertexIndex < OldData.NumPlaneVertices(PlaneIndex); ++PlaneVertexIndex)
					{
						OldPlaneVertices[PlaneIndex][PlaneVertexIndex] = (int32)OldData.GetPlaneVertex(PlaneIndex, PlaneVertexIndex);
					}
				}
				OldNumVertices = OldData.NumVertices();
			}
		};
	}

}
