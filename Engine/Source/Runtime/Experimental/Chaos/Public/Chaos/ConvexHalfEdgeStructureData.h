// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "ChaosArchive.h"
#include "ChaosCheck.h"
#include "ChaosLog.h"
#include "Math/NumericLimits.h"
#include "UObject/PhysicsObjectVersion.h"

namespace Chaos
{
	// Default convex structure index traits - assumes signed
	template<typename T_INDEX>
	struct TConvexStructureIndexTraits
	{
		using FIndex = T_INDEX;
		static const FIndex InvalidIndex = TNumericLimits<FIndex>::Lowest();
		static const FIndex MaxIndex = TNumericLimits<FIndex>::Max();

		static_assert(TIsSigned<T_INDEX>::Value, "The default TConvexStructureIndexTraits implementation is only valid for signed T_INDEX");
	};

	// uint8 uses 255 as InvalidIndex, and therefore supports elements with indices 0...254
	template<>
	struct TConvexStructureIndexTraits<uint8>
	{
		using FIndex = uint8;
		static const FIndex InvalidIndex = TNumericLimits<FIndex>::Max();
		static const FIndex MaxIndex = TNumericLimits<FIndex>::Max() - 1;
	};

	// Convex half-edge structure data.
	// Uses indices rather than pointers.
	// Supports different index sizes.
	template<typename T_INDEX>
	class TConvexHalfEdgeStructureData
	{
	public:
		using FIndex = T_INDEX;
		using FIndexTraits = TConvexStructureIndexTraits<T_INDEX>;
		using FConvexHalfEdgeStructureData = TConvexHalfEdgeStructureData<T_INDEX>;

		static const FIndex InvalidIndex = FIndexTraits::InvalidIndex;
		static const int32 MaxIndex = (int32)FIndexTraits::MaxIndex;

		friend class FVertexPlaneIterator;

		// A plane of a convex hull. Each plane has an array of half edges, stored
		// as an index into the edge list and a count.
		struct FPlaneData
		{
			FIndex FirstHalfEdgeIndex;	// index into HalfEdges
			FIndex NumHalfEdges;

			friend FArchive& operator<<(FArchive& Ar, FPlaneData& Value)
			{
				return Ar << Value.FirstHalfEdgeIndex << Value.NumHalfEdges;
			}
		};

		// Every plane is bounded by a sequence of edges, and every edge should be shared 
		// by two planes. The edges that bound a plane are stored as a sequence of half-edges. 
		// Each half-edge references the starting vertex of the edge, the half-edge 
		// pointing in the opposite direction (belonging to the plane that shares the edge),
		// and the next half-edge on the same plane.
		struct FHalfEdgeData
		{
			FIndex PlaneIndex;			// index into Planes
			FIndex VertexIndex;			// index into Vertices
			FIndex TwinHalfEdgeIndex;	// index into HalfEdges

			friend FArchive& operator<<(FArchive& Ar, FHalfEdgeData& Value)
			{
				return Ar << Value.PlaneIndex << Value.VertexIndex << Value.TwinHalfEdgeIndex;
			}
		};

		// A vertex of a convex hull. We just store one edge that uses the vertex - the others
		// can be found via the half-edge links.
		struct FVertexData
		{
			FIndex FirstHalfEdgeIndex;	// index into HalfEdges

			friend FArchive& operator<<(FArchive& Ar, FVertexData& Value)
			{
				return Ar << Value.FirstHalfEdgeIndex;
			}
		};

		// We cache 3 planes per vertex since this is the most common request and a major bottleneck in GJK 
		// (SupportCoreScaled -> GetMarginAdjustedVertexScaled -> FindVertexPlanes
		struct FVertexPlanes
		{
			FIndex PlaneIndices[3];
			FIndex NumPlaneIndices = 0;
		};

		// List of 3 halfedges per vertex for regular convexes  
		struct FVertexHalfEdges
		{
			FIndex HalfEdgeIndices[3];
			FIndex NumHalfEdgeIndices = 0;
		};

		// Initialize the structure data from the array of vertex indices per plane (in CW or CCW order - it is retained in structure)
		// If this fails for some reason, the structure data will be invalid (check IsValid())
		static FConvexHalfEdgeStructureData MakePlaneVertices(const TArray<TArray<int32>>& InPlaneVertices, int32 InNumVertices)
		{
			FConvexHalfEdgeStructureData StructureData;
			StructureData.SetPlaneVertices(InPlaneVertices, InNumVertices);
			return StructureData;
		}

		// Return true if we can support this convex, based on number of features and maximum index size
		static bool CanMake(const TArray<TArray<int32>>& InPlaneVertices, int32 InNumVertices)
		{
			int32 HalfEdgeCount = 0;
			for (int32 PlaneIndex = 0; PlaneIndex < InPlaneVertices.Num(); ++PlaneIndex)
			{
				HalfEdgeCount += InPlaneVertices[PlaneIndex].Num();
			}

			// For a well-formed convex HalfEdgeCount must be larger than NumPlanes and NumVerts, but check them all anyway just in case...
			return ((HalfEdgeCount <= MaxIndex) && (InPlaneVertices.Num() <= MaxIndex) && (InNumVertices <= MaxIndex));
		}


		bool IsValid() const { return Planes.Num() > 0; }
		int32 NumPlanes() const { return Planes.Num(); }
		int32 NumHalfEdges() const { return HalfEdges.Num(); }
		int32 NumVertices() const { return Vertices.Num(); }

		// Number of unique half-edges (no half edge's twin is in also the list). Should be NumHalfEdges/2
		int32 NumEdges() const { return Edges.Num(); }		

		FPlaneData& GetPlane(int32 PlaneIndex) { return Planes[PlaneIndex]; }
		const FPlaneData& GetPlane(int32 PlaneIndex) const { return Planes[PlaneIndex]; }
		FHalfEdgeData& GetHalfEdge(int32 HalfEdgeIndex) { return HalfEdges[HalfEdgeIndex]; }
		const FHalfEdgeData& GetHalfEdge(int32 HalfEdgeIndex) const { return HalfEdges[HalfEdgeIndex]; }
		int32 GetHalfEdgeIndex(int32 EdgeIndex) const { return Edges[EdgeIndex]; }
		FVertexData& GetVertex(int32 VertexIndex) { return Vertices[VertexIndex]; }
		const FVertexData& GetVertex(int32 VertexIndex) const { return Vertices[VertexIndex]; }

		// The number of edges bounding the specified plane
		int32 NumPlaneHalfEdges(int32 PlaneIndex) const
		{
			return GetPlane(PlaneIndex).NumHalfEdges;
		}

		// The edge index of one of the bounding edges of a plane
		// PlaneIndex must be in range [0, NumPlanes())
		// PlaneEdgeIndex must be in range [0, NumPlaneHalfEdges(PlaneIndex))
		// return value is in range [0, NumHalfEdges())
		int32 GetPlaneHalfEdge(int32 PlaneIndex, int32 PlaneEdgeIndex) const
		{
			check(PlaneEdgeIndex >= 0);
			check(PlaneEdgeIndex < NumPlaneHalfEdges(PlaneIndex));
			return GetPlane(PlaneIndex).FirstHalfEdgeIndex + PlaneEdgeIndex;
		}

		// The number of vertices that bound the specified plane (same as number of half edges)
		// PlaneIndex must be in range [0, NumPlaneHalfEdges(PlaneIndex))
		int32 NumPlaneVertices(int32 PlaneIndex) const
		{
			return GetPlane(PlaneIndex).NumHalfEdges;
		}

		// Get the index of one of the vertices bounding the specified plane
		// PlaneIndex must be in range [0, NumPlanes())
		// PlaneVertexIndex must be in [0, NumPlaneVertices(PlaneIndex))
		// return value is in [0, NumVertices())
		int32 GetPlaneVertex(int32 PlaneIndex, int32 PlaneVertexIndex) const
		{
			const int32 HalfEdgeIndex = GetPlaneHalfEdge(PlaneIndex, PlaneVertexIndex);
			return GetHalfEdge(HalfEdgeIndex).VertexIndex;
		}

		// HalfEdgeIndex must be in range [0, NumHalfEdges())
		// return value is in range [0, NumPlanes())
		int32 GetHalfEdgePlane(int32 HalfEdgeIndex) const
		{
			return GetHalfEdge(HalfEdgeIndex).PlaneIndex;
		}

		// HalfEdgeIndex must be in range [0, NumHalfEdges())
		// return value is in range [0, NumVertices())
		int32 GetHalfEdgeVertex(int32 HalfEdgeIndex) const
		{
			return GetHalfEdge(HalfEdgeIndex).VertexIndex;
		}

		// HalfEdgeIndex must be in range [0, NumHalfEdges())
		// return value is in range [0, NumHalfEdges())
		int32 GetTwinHalfEdge(int32 HalfEdgeIndex) const
		{
			return GetHalfEdge(HalfEdgeIndex).TwinHalfEdgeIndex;
		}

		// Get the previous half edge on the same plane
		// HalfEdgeIndex must be in range [0, NumHalfEdges())
		// return value is in range [0, NumHalfEdges())
		int32 GetPrevHalfEdge(int32 HalfEdgeIndex) const
		{
			// Calculate the edge index on the plane
			const int32 PlaneIndex = GetHalfEdge(HalfEdgeIndex).PlaneIndex;
			const int32 PlaneHalfEdgeIndex = HalfEdgeIndex - GetPlane(PlaneIndex).FirstHalfEdgeIndex;
			return GetPrevPlaneHalfEdge(PlaneIndex, PlaneHalfEdgeIndex);
		}

		// Get the next half edge on the same plane
		// HalfEdgeIndex must be in range [0, NumHalfEdges())
		// return value is in range [0, NumHalfEdges())
		int32 GetNextHalfEdge(int32 HalfEdgeIndex) const
		{
			// Calculate the edge index on the plane
			const int32 PlaneIndex = GetHalfEdge(HalfEdgeIndex).PlaneIndex;
			const int32 PlaneHalfEdgeIndex = HalfEdgeIndex - GetPlane(PlaneIndex).FirstHalfEdgeIndex;
			return GetNextPlaneHalfEdge(PlaneIndex, PlaneHalfEdgeIndex);
		}

		// Get a vertex in the specified Edge (NOTE: edge index, not half-edge index)
		// EdgeIndex must be in range [0, NumEdges())
		// EdgeVertexIndex must be 0 or 1
		// return value is in range [0, NumVertices())
		int32 GetEdgeVertex(int32 EdgeIndex, int32 EdgeVertexIndex) const
		{
			if (EdgeVertexIndex == 0)
			{
				return GetHalfEdgeVertex(Edges[EdgeIndex]);
			}
			else
			{
				return GetHalfEdgeVertex(GetTwinHalfEdge(Edges[EdgeIndex]));
			}
		}

		// Get a plane for the specified Edge (NOTE: edge index, not half-edge index)
		// EdgeIndex must be in range [0, NumEdges())
		// EdgePlaneIndex must be 0 or 1
		// return value is in range [0, NumPlanes())
		int32 GetEdgePlane(int32 EdgeIndex, int32 EdgePlaneIndex) const
		{
			if (EdgePlaneIndex == 0)
			{
				return GetHalfEdgePlane(Edges[EdgeIndex]);
			}
			else
			{
				return GetHalfEdgePlane(GetTwinHalfEdge(Edges[EdgeIndex]));
			}
		}

		// VertexIndex must be in range [0, NumVertices())
		// return value is in range [0, NumHalfEdges())
		int32 GetVertexFirstHalfEdge(int32 VertexIndex) const
		{
			return GetVertex(VertexIndex).FirstHalfEdgeIndex;
		}

		// Iterate over the edges associated with a plane. These edges form the boundary of the plane.
		// Visitor should return false to halt iteration.
		// FVisitorType should be a function with signature: bool(int32 HalfEdgeIndex, int32 NextHalfEdgeIndex) that return false to stop the visit loop
		template<typename FVisitorType>
		inline void VisitPlaneEdges(int32 PlaneIndex, const FVisitorType& Visitor) const
		{
			const int32 FirstHalfEdgeIndex = GetPlane(PlaneIndex).FirstHalfEdgeIndex;
			int32 HalfEdgeIndex0 = FirstHalfEdgeIndex;
			if (HalfEdgeIndex0 != InvalidIndex)
			{
				bool bContinue = true;
				do
				{
					const int32 HalfEdgeIndex1 = GetNextHalfEdge(HalfEdgeIndex0);
					if (HalfEdgeIndex1 != InvalidIndex)
					{
						bContinue = Visitor(HalfEdgeIndex0, HalfEdgeIndex1);
					}
					HalfEdgeIndex0 = HalfEdgeIndex1;
				} while (bContinue && (HalfEdgeIndex0 != FirstHalfEdgeIndex) && (HalfEdgeIndex0 != InvalidIndex));
			}
		}

		// Iterate over the half-edges associated with a vertex (leading out from the vertex, so all half edges have the vertex as the root).
		// Visitor should return false to halt iteration.
		// FVisitorType should be a function with signature: bool(int32 HalfEdgeIndex) that returns false to stop the visit loop.
		template<typename FVisitorType>
		inline void VisitVertexHalfEdges(int32 VertexIndex, const FVisitorType& Visitor) const
		{
			const int32 FirstHalfEdgeIndex = GetVertex(VertexIndex).FirstHalfEdgeIndex;
			int32 HalfEdgeIndex = FirstHalfEdgeIndex;
			if (HalfEdgeIndex != InvalidIndex)
			{
				bool bContinue = true;
				do
				{
					bContinue = Visitor(HalfEdgeIndex);
					const int32 TwinHalfEdgeIndex = GetTwinHalfEdge(HalfEdgeIndex);
					if (TwinHalfEdgeIndex == InvalidIndex)
					{
						break;
					}
					HalfEdgeIndex = GetNextHalfEdge(TwinHalfEdgeIndex);
				} while (bContinue && (HalfEdgeIndex != FirstHalfEdgeIndex) && (HalfEdgeIndex != InvalidIndex));
			}
		}

		// Fill an array with plane indices for the specified vertex. Return the number of planes found.
		int32 FindVertexPlanes(int32 VertexIndex, int32* PlaneIndices, int32 MaxVertexPlanes) const
		{
			int32 NumPlanesFound = 0;

			if (MaxVertexPlanes > 0)
			{
				VisitVertexHalfEdges(VertexIndex,
					[this, PlaneIndices, MaxVertexPlanes, &NumPlanesFound](int32 HalfEdgeIndex)
					{
						PlaneIndices[NumPlanesFound++] = GetHalfEdgePlane(HalfEdgeIndex);
						return (NumPlanesFound < MaxVertexPlanes);
					});
			}

			return NumPlanesFound;
		}

		int32 GetVertexPlanes3(int32 VertexIndex, int32& PlaneIndex0, int32& PlaneIndex1, int32& PlaneIndex2) const
		{
			const FVertexPlanes& VertexPlane = VertexPlanes[VertexIndex];
			PlaneIndex0 = (int32)VertexPlane.PlaneIndices[0];
			PlaneIndex1 = (int32)VertexPlane.PlaneIndices[1];
			PlaneIndex2 = (int32)VertexPlane.PlaneIndices[2];
			return (int32)VertexPlane.NumPlaneIndices;
		}

		// Build half edge structure datas for regular convexes with exactly 3 faces per vertex
		bool BuildRegularDatas(const TArray<TArray<int32>>& InPlaneVertices, int32 InNumVertices)
		{
			// Count the edges
			int32 HalfEdgeCount = 0;
			for (int32 PlaneIndex = 0; PlaneIndex < InPlaneVertices.Num(); ++PlaneIndex)
			{
				HalfEdgeCount += InPlaneVertices[PlaneIndex].Num();
			}

			if ((InPlaneVertices.Num() > MaxIndex) || (HalfEdgeCount > MaxIndex) || (InNumVertices > MaxIndex))
			{
				// We should never get here. See GetRequiredIndexType::GetRequiredIndexType which calls CanMake() on eachn index type until it fits
				UE_LOG(LogChaos, Error, TEXT("Unable to create structure data for convex. MaxIndex too small (%" SIZE_T_FMT " bytes) for Planes: %d HalfEdges: %d Verts: %d"), sizeof(FIndex), InPlaneVertices.Num(), HalfEdgeCount, InNumVertices);
				return false;
			}

			Planes.SetNum(InPlaneVertices.Num());
			HalfEdges.SetNum(HalfEdgeCount);
			Vertices.SetNum(InNumVertices);
			VertexPlanes.SetNum(InNumVertices);

			TArray<FVertexHalfEdges> VertexHalfEdges;
			TArray<FIndex> HalfEdgeVertices;
			
			VertexHalfEdges.SetNum(InNumVertices);
			HalfEdgeVertices.SetNum(HalfEdgeCount);

			// Initialize the vertex list - it will be filled in as we build the edge list
			for (int32 VertexIndex = 0; VertexIndex < Vertices.Num(); ++VertexIndex)
			{
				GetVertex(VertexIndex).FirstHalfEdgeIndex = InvalidIndex;
			}

			// Build the planes and edges. The edges for a plane are stored sequentially in the half-edge array.
			// On the first pass, the edges contain 2 vertex indices, rather than a vertex index and a twin edge index.
			// We fix this up on a second pass.
			int32 NextHalfEdgeIndex = 0;
			for (int32 PlaneIndex = 0; PlaneIndex < InPlaneVertices.Num(); ++PlaneIndex)
			{
				const TArray<int32>& PlaneVertices = InPlaneVertices[PlaneIndex];

				GetPlane(PlaneIndex) =
				{
					(FIndex)NextHalfEdgeIndex,
					(FIndex)PlaneVertices.Num()
				};

				for (int32 PlaneVertexIndex = 0; PlaneVertexIndex < PlaneVertices.Num(); ++PlaneVertexIndex)
				{
					// Add a new edge
					const int32 VertexIndex0 = PlaneVertices[PlaneVertexIndex];
					const int32 VertexIndex1 = PlaneVertices[(PlaneVertexIndex + 1) % PlaneVertices.Num()];
					GetHalfEdge(NextHalfEdgeIndex) =
					{
						(FIndex)PlaneIndex,
						(FIndex)VertexIndex0,
						(FIndex)VertexIndex1,	// Will get converted to a half-edge index later
					};
					HalfEdgeVertices[NextHalfEdgeIndex] = (FIndex)VertexIndex1;

					// If this is the first time Vertex0 has showed up, set its edge index
					// For valid and regular convexes each vertices have 2 halfedges starting from itself
					if (Vertices[VertexIndex0].FirstHalfEdgeIndex == InvalidIndex)
					{
						Vertices[VertexIndex0].FirstHalfEdgeIndex = (FIndex)NextHalfEdgeIndex;
					}
					
					// For regular convexes each vertices have exactly 3 halfedges
                    VertexHalfEdges[VertexIndex0].HalfEdgeIndices[VertexHalfEdges[VertexIndex0].NumHalfEdgeIndices++] = (FIndex)NextHalfEdgeIndex;

					// For regular convexes each vertices have exactly 3 planes
					VertexPlanes[VertexIndex0].PlaneIndices[VertexPlanes[VertexIndex0].NumPlaneIndices++] = (FIndex)PlaneIndex;

					++NextHalfEdgeIndex;
				}
			}
			Edges.Empty();
			Edges.Reserve(NumHalfEdges() / 2);
			
			for (int32 HalfEdgeIndex = 0; HalfEdgeIndex < HalfEdges.Num(); ++HalfEdgeIndex)
			{
				const int32 VertexIndex0 = HalfEdges[HalfEdgeIndex].VertexIndex;
				const int32 VertexIndex1 = HalfEdges[HalfEdgeIndex].TwinHalfEdgeIndex;

				for(int32 VertexHalfEdgeIndex = 0, NumHalfEdgeIndices = VertexHalfEdges[VertexIndex1].NumHalfEdgeIndices; 
				                     VertexHalfEdgeIndex < NumHalfEdgeIndices; ++VertexHalfEdgeIndex)
				{
				    const FIndex TwinHalfEdgeIndex = VertexHalfEdges[VertexIndex1].HalfEdgeIndices[VertexHalfEdgeIndex];
					if(HalfEdgeVertices[TwinHalfEdgeIndex] == VertexIndex0)
					{
						HalfEdges[HalfEdgeIndex].TwinHalfEdgeIndex = TwinHalfEdgeIndex;
						break;
					}
				}

				if(VertexIndex0 < HalfEdges[HalfEdges[HalfEdgeIndex].TwinHalfEdgeIndex].VertexIndex)
				{
					Edges.Add((FIndex)HalfEdgeIndex);
				}
			}
			return true;
		}
		

		// Initialize the structure data from the set of vertices associated with each plane.
		// The vertex indices are assumed to be in CCW order (or CW order - doesn't matter here
		// as long as it is sequential).
		bool SetPlaneVertices(const TArray<TArray<int32>>& InPlaneVertices, int32 InNumVertices)
		{
			// Count the edges
			int32 HalfEdgeCount = 0;
			for (int32 PlaneIndex = 0; PlaneIndex < InPlaneVertices.Num(); ++PlaneIndex)
			{
				HalfEdgeCount += InPlaneVertices[PlaneIndex].Num();
			}

			if ((InPlaneVertices.Num() > MaxIndex) || (HalfEdgeCount > MaxIndex) || (InNumVertices > MaxIndex))
			{
				// We should never get here. See GetRequiredIndexType::GetRequiredIndexType which calls CanMake() on eachn index type until it fits
				UE_LOG(LogChaos, Error, TEXT("Unable to create structure data for convex. MaxIndex too small (%" SIZE_T_FMT " bytes) for Planes: %d HalfEdges: %d Verts: %d"), sizeof(FIndex), InPlaneVertices.Num(), HalfEdgeCount, InNumVertices);
				return false;
			}

			Planes.SetNum(InPlaneVertices.Num());
			HalfEdges.SetNum(HalfEdgeCount);
			Vertices.SetNum(InNumVertices);

			// Initialize the vertex list - it will be filled in as we build the edge list
			for (int32 VertexIndex = 0; VertexIndex < Vertices.Num(); ++VertexIndex)
			{
				GetVertex(VertexIndex).FirstHalfEdgeIndex = InvalidIndex;
			}

			// Build the planes and edges. The edges for a plane are stored sequentially in the half-edge array.
			// On the first pass, the edges contain 2 vertex indices, rather than a vertex index and a twin edge index.
			// We fix this up on a second pass.
			int32 NextHalfEdgeIndex = 0;
			for (int32 PlaneIndex = 0; PlaneIndex < InPlaneVertices.Num(); ++PlaneIndex)
			{
				const TArray<int32>& PlaneVertices = InPlaneVertices[PlaneIndex];

				GetPlane(PlaneIndex) =
				{
					(FIndex)NextHalfEdgeIndex,
					(FIndex)PlaneVertices.Num()
				};

				for (int32 PlaneVertexIndex = 0; PlaneVertexIndex < PlaneVertices.Num(); ++PlaneVertexIndex)
				{
					// Add a new edge
					const int32 VertexIndex0 = PlaneVertices[PlaneVertexIndex];
					const int32 VertexIndex1 = PlaneVertices[(PlaneVertexIndex + 1) % PlaneVertices.Num()];
					GetHalfEdge(NextHalfEdgeIndex) =
					{
						(FIndex)PlaneIndex,
						(FIndex)VertexIndex0,
						(FIndex)VertexIndex1,	// Will get converted to a half-edge index later
					};

					// If this is the first time Vertex0 has showed up, set its edge index
					if (Vertices[VertexIndex0].FirstHalfEdgeIndex == InvalidIndex)
					{
						Vertices[VertexIndex0].FirstHalfEdgeIndex = (FIndex)NextHalfEdgeIndex;
					}

					++NextHalfEdgeIndex;
				}
			}

			// Find the twin half edge for each edge
			// NOTE: we have to deal with mal-formed convexes which claim to have edges that use the
			// same vertex pair in the same order.
			// @todo(chaos): track down to source of the mal-formed convexes
			// @todo(chaos): could use a map of vertex-index-pair to half edge to eliminate O(N^2) algorithm
			TArray<bool> HalfEdgeTwinned;
			TArray<FIndex> TwinHalfEdgeIndices;
			TwinHalfEdgeIndices.SetNum(HalfEdges.Num());
			HalfEdgeTwinned.SetNum(HalfEdges.Num());
			for (int32 HalfEdgeIndex = 0; HalfEdgeIndex < TwinHalfEdgeIndices.Num(); ++HalfEdgeIndex)
			{
				TwinHalfEdgeIndices[HalfEdgeIndex] = InvalidIndex;
				HalfEdgeTwinned[HalfEdgeIndex] = false;
			}
			for (int32 HalfEdgeIndex0 = 0; HalfEdgeIndex0 < HalfEdges.Num(); ++HalfEdgeIndex0)
			{
				const int32 VertexIndex0 = HalfEdges[HalfEdgeIndex0].VertexIndex;
				const int32 VertexIndex1 = HalfEdges[HalfEdgeIndex0].TwinHalfEdgeIndex;	// Actually a vertex index for now...

				// Find the edge with the vertices the other way round
				for (int32 HalfEdgeIndex1 = 0; HalfEdgeIndex1 < HalfEdges.Num(); ++HalfEdgeIndex1)
				{
					if ((HalfEdges[HalfEdgeIndex1].VertexIndex == VertexIndex1) && (HalfEdges[HalfEdgeIndex1].TwinHalfEdgeIndex == VertexIndex0))
					{
						// We deal with edge duplication by leaving a half edge without a twin
						if (!HalfEdgeTwinned[HalfEdgeIndex1])
						{
							TwinHalfEdgeIndices[HalfEdgeIndex0] = (FIndex)HalfEdgeIndex1;
							HalfEdgeTwinned[HalfEdgeIndex1] = true;
						}
						else
						{
							TwinHalfEdgeIndices[HalfEdgeIndex0] = InvalidIndex;
						}
						break;
					}
				}
			}

			// Set the twin edge indices
			for (int32 HalfEdgeIndex = 0; HalfEdgeIndex < HalfEdges.Num(); ++HalfEdgeIndex)
			{
				GetHalfEdge(HalfEdgeIndex).TwinHalfEdgeIndex = (FIndex)TwinHalfEdgeIndices[HalfEdgeIndex];
			}

			BuildVertexPlanes();

			BuildUniqueEdgeList();

			return true;
		}

		void Serialize(FArchive& Ar)
		{
			Ar.UsingCustomVersion(FPhysicsObjectVersion::GUID);

			Ar << Planes;
			Ar << HalfEdges;
			Ar << Vertices;

			const bool bHasUniqueEdgeList = Ar.CustomVer(FPhysicsObjectVersion::GUID) >= FPhysicsObjectVersion::ChaosConvexHasUniqueEdgeSet;
			if (bHasUniqueEdgeList)
			{
				Ar << Edges;
			}

			// Handle older data without the edge list and also an issue where some assets were saved without having their EdgeList generated (now fixed)
			// Avoid adding a new custom version for that fix because we want to integrate this into other streams
			if (Ar.IsLoading() && (Edges.Num() == 0) && (HalfEdges.Num() > 0))
			{
				BuildUniqueEdgeList();
				ensureMsgf(Edges.Num() > 0, TEXT("Invalid edge data on convex in Load"));
			}

			if (Ar.IsLoading())
			{
				BuildVertexPlanes();
			}
		}

		friend FArchive& operator<<(FArchive& Ar, FConvexHalfEdgeStructureData& Value)
		{
			Value.Serialize(Ar);
			return Ar;
		}

#if INTEL_ISPC
		// See PerParticlePBDCollisionConstraint.cpp
		// ISPC code has matching structs for interpreting FImplicitObjects.
		// This is used to verify that the structs stay the same.
		struct FISPCDataVerifier
		{
			static constexpr int32 OffsetOfPlanes() { return offsetof(TConvexHalfEdgeStructureData, Planes); }
			static constexpr int32 SizeOfPlanes() { return sizeof(TConvexHalfEdgeStructureData::Planes); }
			static constexpr int32 OffsetOfHalfEdges() { return offsetof(TConvexHalfEdgeStructureData, HalfEdges); }
			static constexpr int32 SizeOfHalfEdges() { return sizeof(TConvexHalfEdgeStructureData::HalfEdges); }
			static constexpr int32 OffsetOfVertices() { return offsetof(TConvexHalfEdgeStructureData, Vertices); }
			static constexpr int32 SizeOfVertices() { return sizeof(TConvexHalfEdgeStructureData::Vertices); }
			static constexpr int32 OffsetOfEdges() { return offsetof(TConvexHalfEdgeStructureData, Edges); }
			static constexpr int32 SizeOfEdges() { return sizeof(TConvexHalfEdgeStructureData::Edges); }
			static constexpr int32 OffsetOfVertexPlanes() { return offsetof(TConvexHalfEdgeStructureData, VertexPlanes); }
			static constexpr int32 SizeOfVertexPlanes() { return sizeof(TConvexHalfEdgeStructureData::VertexPlanes); }
		};
		friend FISPCDataVerifier;
#endif // #if INTEL_ISPC

	private:

		// The edge index of the previous edge on the plane (loops)
		// PlaneIndex must be in range [0, NumPlanes())
		// PlaneHalfEdgeIndex must be in range [0, NumPlaneHalfEdges(PlaneIndex))
		// return value is in range [0, NumHalfEdges())
		int32 GetPrevPlaneHalfEdge(int32 PlaneIndex, int32 PlaneHalfEdgeIndex) const
		{
			// A plane's edges are sequential and loop
			check(PlaneHalfEdgeIndex >= 0);
			check(PlaneHalfEdgeIndex < NumPlaneHalfEdges(PlaneIndex));
			const int32 PlaneHalfEdgeCount = NumPlaneHalfEdges(PlaneIndex);
			const int32 PrevPlaneHalfEdgeIndex = (PlaneHalfEdgeIndex + PlaneHalfEdgeCount - 1) % PlaneHalfEdgeCount;
			return GetPlaneHalfEdge(PlaneIndex, PrevPlaneHalfEdgeIndex);
		}

		// The edge index of the next edge on the plane (loops)
		// PlaneIndex must be in range [0, NumPlanes())
		// PlaneHalfEdgeIndex must be in range [0, NumPlaneHalfEdges(PlaneIndex))
		// return value is in range [0, NumHalfEdges())
		int32 GetNextPlaneHalfEdge(int32 PlaneIndex, int32 PlaneHalfEdgeIndex) const
		{
			// A plane's edges are sequential and loop
			check(PlaneHalfEdgeIndex >= 0);
			check(PlaneHalfEdgeIndex < NumPlaneHalfEdges(PlaneIndex));
			const int32 PlaneHalfEdgeCount = NumPlaneHalfEdges(PlaneIndex);
			const int32 NextPlaneHalfEdgeIndex = (PlaneHalfEdgeIndex + 1) % PlaneHalfEdgeCount;
			return GetPlaneHalfEdge(PlaneIndex, NextPlaneHalfEdgeIndex);
		}

		// Generate the set of half-edges where none of the edge twins are also in the list.
		// this is effectively the set of full edges. This will also exclude any malformed edges if they exist.
		void BuildUniqueEdgeList()
		{
			Edges.Empty();
			Edges.Reserve(NumHalfEdges() / 2);

			for (int32 HalfEdgeIndex = 0; HalfEdgeIndex < NumHalfEdges(); ++HalfEdgeIndex)
			{
				const FHalfEdgeData& Edge = GetHalfEdge(HalfEdgeIndex);
				if (Edge.TwinHalfEdgeIndex != InvalidIndex)
				{
					const FHalfEdgeData& TwinEdge = GetHalfEdge(Edge.TwinHalfEdgeIndex);
					if ((Edge.VertexIndex != InvalidIndex) && (TwinEdge.VertexIndex != InvalidIndex))
					{
						if (Edge.VertexIndex < TwinEdge.VertexIndex)
						{
							Edges.Add((FIndex)HalfEdgeIndex);
						}
					}
				}
			}
		}

		void BuildVertexPlanes()
		{
			VertexPlanes.SetNum(Vertices.Num());

			for (int32 VertexIndex = 0; VertexIndex < NumVertices(); ++VertexIndex)
			{
				FVertexPlanes& VertexPlane = VertexPlanes[VertexIndex];
				VertexPlane.NumPlaneIndices = 0;
				VertexPlane.PlaneIndices[0] = FIndex(INDEX_NONE);
				VertexPlane.PlaneIndices[1] = FIndex(INDEX_NONE);
				VertexPlane.PlaneIndices[2] = FIndex(INDEX_NONE);

				const int32 FirstHalfEdgeIndex = GetVertex(VertexIndex).FirstHalfEdgeIndex;
				int32 HalfEdgeIndex = FirstHalfEdgeIndex;
				if (HalfEdgeIndex != InvalidIndex)
				{
					do
					{
						if(VertexPlane.NumPlaneIndices < (FIndex)UE_ARRAY_COUNT(VertexPlane.PlaneIndices))
						{
							VertexPlane.PlaneIndices[VertexPlane.NumPlaneIndices] = (FIndex)GetHalfEdgePlane(HalfEdgeIndex);
						}

						// Caching of the Max number of plane indices on this vertex (could be higher than 3)
						// This can be used to determine if a call to GetVertexPlanes3 will actually return all the planes
						// that use a particular vertex. This is useful in collision detection for box-like objects
						// to avoid calling FindVertexPlanes
						++VertexPlane.NumPlaneIndices;

						// If we hit this there's a mal-formed convex case that we did not detect (we should be dealing with all cases - see SetPlaneVertices)
						if (!ensure(VertexPlane.NumPlaneIndices <= Planes.Num()))
						{
							VertexPlane.NumPlaneIndices = 0;
							break;
						}

						const int32 TwinHalfEdgeIndex = GetTwinHalfEdge(HalfEdgeIndex);
						if (TwinHalfEdgeIndex == InvalidIndex)
						{
							break;
						}
						HalfEdgeIndex = GetNextHalfEdge(TwinHalfEdgeIndex);
					} while ((HalfEdgeIndex != FirstHalfEdgeIndex) && (HalfEdgeIndex != InvalidIndex));
				}
			}
		}

		TArray<FPlaneData> Planes;
		TArray<FHalfEdgeData> HalfEdges;
		TArray<FVertexData> Vertices;
		TArray<FIndex> Edges;
		TArray<FVertexPlanes> VertexPlanes;
	};


	// Typedefs for the supported index sizes
	using FConvexHalfEdgeStructureDataS32 = TConvexHalfEdgeStructureData<int32>;
	using FConvexHalfEdgeStructureDataS16 = TConvexHalfEdgeStructureData<int16>;
	using FConvexHalfEdgeStructureDataU8 = TConvexHalfEdgeStructureData<uint8>;
}
