// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/Core.h"
#include "Chaos/ImplicitObject.h"
#include "Chaos/Tribox.h"

namespace Chaos
{
	
namespace Private
{
	class FImplicitBVHObject;
	struct FCollisionObjects
	{
		TArray<Private::FImplicitBVHObject> ImplicitObjects;
	};

	/**
	 * @brief The convex optimizer goal is to have a central place where
	 * implicits hierarchy could be modified in order to accelerate collision detection
	 */
	class FConvexOptimizer
	{
		public :
		
		struct FTriboxNode
		{
			using FValidEdges = TMap<FImplicitObject*, bool>;
			
			// Cached tribox
			Private::FTribox NodeTribox;

			// Cached convex
			FImplicitObjectPtr TriboxConvex = nullptr;

			// Shape Index
			int32 ShapeIndex = INDEX_NONE;
			
			// Node volume
			FTribox::FRealType NodeVolume = 0.0f;

			// Valid flag to check if the node has already been processed during merging
			bool bValidNode = true;

			// Valid flag to check if the edge has already been processed during merging
			FValidEdges bValidEdges = {};

			// Convex leaf offset that was used when the convex has been built
			int32 ConvexId = 0;
		};

		using FTriboxNodes = TMap<FImplicitObject*, FTriboxNode>;
		
		CHAOS_API FConvexOptimizer();

		// Default destructor
		CHAOS_API ~FConvexOptimizer();

		// Simplify all the convexes in the hierarchy 
		CHAOS_API void SimplifyRootConvexes(const Chaos::FImplicitObjectUnionPtr& UnionGeometry, const FShapesArray& UnionShapes,
			const EObjectStateType ObjectState, const TBitArray<>& bOptimizeConvexes);

		// Check if the manager is valid or not
		bool IsValid() const {return !SimplifiedConvexes.IsEmpty();}

		// Visit all the collision objects if they exist / otherwise forward it to the RootHierarchy
		CHAOS_API void VisitCollisionObjects(const FImplicitHierarchyVisitor& VisitorFunc) const;

		// Visit all the overlapping objects if they exist / otherwise forward it to the RootHierarchy
		CHAOS_API void VisitOverlappingObjects(const FAABB3& LocalBounds, const FImplicitHierarchyVisitor& VisitorFunc) const;

		// Get the shapes array 
		const FShapeInstanceArray& GetShapeInstances() const { return ShapesArray;}
		
		// Get the number of collision objects
		CHAOS_API int32 NumCollisionObjects() const; 

	private:

		// Build union connectivity for merging
		void BuildUnionConnectivity(const Chaos::FImplicitObjectUnionPtr& UnionGeometry);

		// Merge the connected shapes
		void MergeConnectedShapes(const Chaos::FImplicitObjectUnionPtr& UnionGeometry, TArray<FTriboxNode>& MergedNodes);

		// Build a single convex
		void BuildSingleConvex(const Chaos::FImplicitObjectUnionPtr& UnionGeometry, const FShapesArray& UnionShapes, const TBitArray<>& bOptimizeConvexes);

		// Build several convexes 
		void BuildMultipleConvex(const Chaos::FImplicitObjectUnionPtr& UnionGeometry, const FShapesArray& UnionShapes, const bool bEnableMerging, const TBitArray<>& bOptimizeConvexes);

		// Build the simplified shapes
		void BuildConvexShapes(const FShapesArray& UnionShapes);

		// List of simplified convexes
		TArray<FImplicitObjectPtr> SimplifiedConvexes;

		// List of all the collision objects to avoid traversing all the hierarchy during midphase
		TUniquePtr<Private::FCollisionObjects> CollisionObjects;

		// Additional shapes array that could be used during collision midphase
		FShapeInstanceArray ShapesArray;

		// Intermediate root triboxes to reuse the intermediate computation
		FTriboxNodes RootTriboxes;

		// BVH used to accelerate the collisions queries
		TUniquePtr<Private::FImplicitBVH> BVH;

		// Main tribox built from all the convexes
		FTriboxNode MainTribox;

		// Leaf offset to have a unique id for the midphase
		int32 NextConvexId = 0;
	};

	// Visit all the collision objects if they exist / otherwise forward it to the RootHierarchy
	void VisitCollisionObjects(const FConvexOptimizer* ConvexOptimizer, const FImplicitObject* ImplicitObject, const FImplicitHierarchyVisitor& VisitorFunc);

	// Visit all the overlapping objects if they exist / otherwise forward it to the RootHierarchy
	void VisitOverlappingObjects(const FConvexOptimizer* ConvexOptimizer, const FImplicitObject* ImplicitObject, const FAABB3& LocalBounds, const FImplicitHierarchyVisitor& VisitorFunc);


}
}
