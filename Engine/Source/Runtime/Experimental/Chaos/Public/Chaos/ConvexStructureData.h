// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/ConvexFlattenedArrayStructureData.h"
#include "Chaos/ConvexHalfEdgeStructureData.h"
#include "ChaosArchive.h"
#include "ChaosCheck.h"
#include "ChaosLog.h"
#include "UObject/PhysicsObjectVersion.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

namespace Chaos
{

	// Metadata for a convex shape used by the manifold generation system and anything
	// else that can benefit from knowing which vertices are associated with the faces.
	class FConvexStructureData
	{
	public:
		using FConvexStructureDataLarge = FConvexHalfEdgeStructureDataS32;
		using FConvexStructureDataMedium = FConvexHalfEdgeStructureDataS16;
		using FConvexStructureDataSmall = FConvexHalfEdgeStructureDataU8;

		// Note: this is serialized (do not change order without adding a new ObjectVersion and legacy load path)
		enum class EIndexType : int8
		{
			None,
			Small,
			Medium,
			Large,
		};

		// Cats the inner data container to the correct index size
		// Do not use - only public for unit tests
		const FConvexStructureDataLarge& DataL() const { checkSlow(IndexType == EIndexType::Large); return *Data.DataL; }
		const FConvexStructureDataMedium& DataM() const { checkSlow(IndexType == EIndexType::Medium); return *Data.DataM; }
		const FConvexStructureDataSmall& DataS() const { checkSlow(IndexType == EIndexType::Small); return *Data.DataS; }

	private:
		FConvexStructureDataLarge& DataL() { checkSlow(IndexType == EIndexType::Large); return *Data.DataL; }
		FConvexStructureDataMedium& DataM() { checkSlow(IndexType == EIndexType::Medium); return *Data.DataM; }
		FConvexStructureDataSmall& DataS() { checkSlow(IndexType == EIndexType::Small); return *Data.DataS; }

		// Call a function on the read-only data (cast to Small, Medium or Large indices as appropriate).
		// It will call "Op" on the down-casted data container if we have a valid container. It will
		// call EmptyOp if the container has not been set up (IndexType is None).
		// Note: This expands into a switch statement, calling the lambda with a container of each index size.
		//
		// Must be used with an auto lambda to handle the downcasting, e.g.:
		//		int32 Plane0VertexCount = ConstDataOp(
		//			[](const auto& ConcreteData) { return ConcreteData.NumPlaneVertices(0); },
		//			[]() { return 0; });
		//
		template<typename T_OP, typename T_EMPTYOP>
		inline auto ConstDataOp(const T_OP& Op, const T_EMPTYOP& EmptyOp) const
		{
			switch (IndexType)
			{
			case EIndexType::Small:
				return Op(DataS());
			case EIndexType::Medium:
				return Op(DataM());
			case EIndexType::Large:
				return Op(DataL());
			}
			return EmptyOp();
		}

		#define CHAOS_CONVEXSTRUCTUREDATA_DATAOP(Op, DefaultValue)	\
			switch (IndexType)										\
			{														\
			case EIndexType::Small:									\
				return Op(DataS());									\
			case EIndexType::Medium:								\
				return Op(DataM());									\
			case EIndexType::Large:									\
				return Op(DataL());									\
			}														\
			return DefaultValue;

		// A write-enabled version of ConstDataOp
		template<typename T_OP, typename T_EMPTYOP>
		inline auto NonConstDataOp(const T_OP& Op, const T_EMPTYOP& EmptyOp)
		{
			switch (IndexType)
			{
			case EIndexType::Small:
				return Op(DataS());
			case EIndexType::Medium:
				return Op(DataM());
			case EIndexType::Large:
				return Op(DataL());
			}
			return EmptyOp();
		}

	public:

		FConvexStructureData()
			: Data{ nullptr }
			, IndexType(EIndexType::None)
		{
		}

		FConvexStructureData(const FConvexStructureData& Other) = delete;

		FConvexStructureData(FConvexStructureData&& Other)
		{
			*this = MoveTemp(Other);
		}

		~FConvexStructureData()
		{
			DestroyDataContainer();
		}

		FConvexStructureData& operator=(const FConvexStructureData& Other) = delete;

		FConvexStructureData& operator=(FConvexStructureData&& Other)
		{
			Data.Ptr = Other.Data.Ptr;
			IndexType = Other.IndexType;

			Other.Data.Ptr = nullptr;
			Other.IndexType = EIndexType::None;

			return *this;
		}
		
		void CopyFrom(const FConvexStructureData& Other)
		{
			DestroyDataContainer();

			check(Data.Ptr == nullptr);
			check(IndexType == EIndexType::None);

			if (Other.IndexType == EIndexType::Large)
			{
				check(Other.Data.DataL);
				Data.DataL = new FConvexStructureDataLarge(*Other.Data.DataL);
			}
			else if (Other.IndexType == EIndexType::Medium)
			{
				check(Other.Data.DataM);
				Data.DataM = new FConvexStructureDataMedium(*Other.Data.DataM);
			}
			else if (Other.IndexType == EIndexType::Small)
			{
				check(Other.Data.DataS);
				Data.DataS = new FConvexStructureDataSmall(*Other.Data.DataS);
			}

			IndexType = Other.IndexType;
		}
		
		inline bool IsValid() const
		{
			return (Data.Ptr != nullptr);
		}

		inline EIndexType GetIndexType() const
		{
			return IndexType;
		}

		inline int32 FindVertexPlanes(int32 VertexIndex, int32* VertexPlanes, int32 MaxVertexPlanes) const
		{
			return ConstDataOp(
				[VertexIndex, VertexPlanes, MaxVertexPlanes](const auto& ConcreteData) { return ConcreteData.FindVertexPlanes(VertexIndex, VertexPlanes, MaxVertexPlanes); },
				[]() { return 0; });
		}

		inline int32 GetVertexPlanes3(int32 VertexIndex, int32& PlaneIndex0, int32& PlaneIndex1, int32& PlaneIndex2) const
		{
			switch (IndexType)
			{
			case EIndexType::Small:
				return DataS().GetVertexPlanes3(VertexIndex, PlaneIndex0, PlaneIndex1, PlaneIndex2);
			case EIndexType::Medium:
				return DataM().GetVertexPlanes3(VertexIndex, PlaneIndex0, PlaneIndex1, PlaneIndex2);
			case EIndexType::Large:
				return DataL().GetVertexPlanes3(VertexIndex, PlaneIndex0, PlaneIndex1, PlaneIndex2);
			}
			return 0;
		}

		// The number of vertices that make up the corners of the specified face
		inline int32 NumPlaneVertices(int32 PlaneIndex) const
		{
			return ConstDataOp(
				[PlaneIndex](const auto& ConcreteData) { return ConcreteData.NumPlaneVertices(PlaneIndex); },
				[]() { return 0; });
		}

		// Get the vertex index (in the outer convex container) of one of the vertices making up the corners of the specified face
		inline int32 GetPlaneVertex(int32 PlaneIndex, int32 PlaneVertexIndex) const
		{
			checkSlow(IsValid());
			return ConstDataOp(
				[PlaneIndex, PlaneVertexIndex](const auto& ConcreteData) { return ConcreteData.GetPlaneVertex(PlaneIndex, PlaneVertexIndex); },
				[]() { return (int32)INDEX_NONE; });
		}


		inline int32 NumHalfEdges() const
		{
			return ConstDataOp(
				[](const auto& ConcreteData) { return ConcreteData.NumHalfEdges(); },
				[]() { return 0; });
		}

		inline int32 NumEdges() const
		{
			return ConstDataOp(
				[](const auto& ConcreteData) { return ConcreteData.NumEdges(); },
				[]() { return 0; });
		}

		inline int32 GetEdgeVertex(int32 EdgeIndex, int32 EdgeVertexIndex) const
		{
			return ConstDataOp(
				[EdgeIndex, EdgeVertexIndex](const auto& ConcreteData)
				{
					return ConcreteData.GetEdgeVertex(EdgeIndex, EdgeVertexIndex);
				},
				[]() { return (int32)INDEX_NONE; });
		}

		inline int32 GetEdgePlane(int32 EdgeIndex, int32 EdgePlaneIndex) const
		{
			return ConstDataOp(
				[EdgeIndex, EdgePlaneIndex](const auto& ConcreteData)
				{
					return ConcreteData.GetEdgePlane(EdgeIndex, EdgePlaneIndex);
				},
				[]() { return (int32)INDEX_NONE; });
		}

		inline int32 GetPlaneHalfEdge(int32 PlaneIndex, int32 FaceVertexIndex) const
		{
			return ConstDataOp(
				[PlaneIndex, FaceVertexIndex](const auto& ConcreteData)
				{
					return ConcreteData.GetPlaneHalfEdge(PlaneIndex, FaceVertexIndex);
				},
				[]() { return (int32)INDEX_NONE; });
		}

		inline void GetHalfEdges(int32 EdgeIndex, int32 &OutEdgeIndex0, int32 &OutEdgeIndex1) const
		{
			return ConstDataOp(
				[EdgeIndex, &OutEdgeIndex0, &OutEdgeIndex1](const auto& ConcreteData)
				{
					OutEdgeIndex0 = ConcreteData.GetHalfEdgeIndex(EdgeIndex);
					OutEdgeIndex1 = ConcreteData.GetTwinHalfEdge(OutEdgeIndex0);
					return; 
				},
				[&OutEdgeIndex0, &OutEdgeIndex1]()
				{ 
					OutEdgeIndex0 = INDEX_NONE;
					OutEdgeIndex1 = INDEX_NONE;
					return; 
				});
		}

		// Initialize the structure data from the set of vertices for each face of the convex
		bool SetPlaneVertices(const TArray<TArray<int32>>& InPlaneVertices, int32 NumVerts, const bool bRegularDatas = false)
		{
			const EIndexType NewIndexType = GetRequiredIndexType(InPlaneVertices, NumVerts);
			CreateDataContainer(NewIndexType);

			return NonConstDataOp(
				[InPlaneVertices, NumVerts, bRegularDatas](auto& ConcreteData) {
				return bRegularDatas ? ConcreteData.BuildRegularDatas(InPlaneVertices, NumVerts) : ConcreteData.SetPlaneVertices(InPlaneVertices, NumVerts); 
					},
				[]() { return true; });
		}

		void Serialize(FArchive& Ar)
		{
			Ar.UsingCustomVersion(FPhysicsObjectVersion::GUID);
			Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

			const bool bUseHalfEdgeStructureData = Ar.CustomVer(FPhysicsObjectVersion::GUID) >= FPhysicsObjectVersion::ChaosConvexUsesHalfEdges;
			
			// Load and convert the legacy structure if necessary
			if (Ar.IsLoading() && !bUseHalfEdgeStructureData)
			{
				LoadLegacyData(Ar);
				return;
			}

			if (Ar.IsLoading())
			{
				// Create the container with the appropriate index size
				EIndexType NewIndexType;
				Ar << NewIndexType;
				CreateDataContainer(NewIndexType);
			}
			else
			{
				Ar << IndexType;
			}

			// Serialize the container with the correct index type
			NonConstDataOp(
				[&Ar](auto& ConcreteData) { Ar << ConcreteData; },
				[]() {});
		}

		friend FArchive& operator<<(FArchive& Ar, FConvexStructureData& Value)
		{
			Value.Serialize(Ar);
			return Ar;
		}

#if INTEL_ISPC
		// See PerParticlePBDCollisionConstraint.cpp
		// ISPC code has matching structs for interpreting FImplicitObjects.
		// This is used to verify that the structs stay the same.
		struct FISPCDataVerifier
		{
			static constexpr int32 OffsetOfData() { return offsetof(FConvexStructureData, Data); }
			static constexpr int32 SizeOfData() { return sizeof(FConvexStructureData::Data); }
			static constexpr int32 OffsetOfIndexType() { return offsetof(FConvexStructureData, IndexType); }
			static constexpr int32 SizeOfIndexType() { return sizeof(FConvexStructureData::IndexType); }
		};
		friend FISPCDataVerifier;
#endif // #if INTEL_ISPC

	private:

		// Load data from an asset saved before we had a proper half-edge data structure
		void LoadLegacyData(FArchive& Ar)
		{
			TArray<TArray<int32>> OldPlaneVertices;
			int32 OldNumVertices = 0;

			Legacy::FLegacyConvexStructureDataLoader::Load(Ar, OldPlaneVertices, OldNumVertices);

			SetPlaneVertices(OldPlaneVertices, OldNumVertices);
		}

		// Determine the minimum index size we need for the specified convex size
		EIndexType GetRequiredIndexType(const TArray<TArray<int32>>& InPlaneVertices, int32 NumVerts)
		{
			if (FConvexStructureDataSmall::CanMake(InPlaneVertices, NumVerts))
			{
				return EIndexType::Small;
			}
			else if (FConvexStructureDataMedium::CanMake(InPlaneVertices, NumVerts))
			{
				return EIndexType::Medium;
			}
			else
			{
				return EIndexType::Large;
			}
		}

		// Create the container to match the desired index size
		void CreateDataContainer(EIndexType InIndexType)
		{
			DestroyDataContainer();

			check(Data.Ptr == nullptr);
			check(IndexType == EIndexType::None);

			if (InIndexType == EIndexType::Large)
			{
				Data.DataL = new FConvexStructureDataLarge();
			}
			else if (InIndexType == EIndexType::Medium)
			{
				Data.DataM = new FConvexStructureDataMedium();
			}
			else if (InIndexType == EIndexType::Small)
			{
				Data.DataS = new FConvexStructureDataSmall();
			}

			IndexType = InIndexType;
		}

		// Destroy the container we created in CreateDataContainer
		void DestroyDataContainer()
		{
			if (Data.Ptr != nullptr)
			{
				check(IndexType != EIndexType::None);

				NonConstDataOp(
					[](auto& ConcreteData) 
					{ 
						delete &ConcreteData;
					},
					[]() {});

				IndexType = EIndexType::None;
				Data.Ptr = nullptr;
			}
		}

		union FStructureData
		{
			void* Ptr;
			FConvexStructureDataLarge* DataL;
			FConvexStructureDataMedium* DataM;
			FConvexStructureDataSmall* DataS;
		};
		FStructureData Data;

		// The index type we require for the structure data
		EIndexType IndexType;
	};

}
