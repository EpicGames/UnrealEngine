// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/ChaosArchive.h"
#include "Containers/Array.h"
#include "Math/UnrealMathUtility.h"

namespace Chaos
{
	struct FCurveKey
	{
		float Time;
		float Value;

		friend FArchive& operator<<(FArchive& Ar, FCurveKey& Key)
		{
			Ar << Key.Time;
			Ar << Key.Value;
			return Ar;
		}
	};

	class FLinearCurve
	{
	public:
		TArray<FCurveKey> Keys;

		FLinearCurve() = default;

		FLinearCurve(std::initializer_list<FCurveKey> InKeys)
			: Keys(InKeys)
		{}

		int32 GetNumKeys() const { return Keys.Num(); };

		// Evaluate the curve at a given time using simple linear interpolation
		float Eval(float InTime) const
		{
			const int32 NumKeys = Keys.Num();
			if (NumKeys == 0)
			{
				return 0.f;
			}

			if (InTime <= Keys[0].Time)
			{
				return Keys[0].Value;
			}

			if (InTime >= Keys[NumKeys - 1].Time)
			{
				return Keys[NumKeys - 1].Value;
			}

			// Linear interpolation between nearest keys
			for (int32 i = 1; i < NumKeys; ++i)
			{
				if (InTime < Keys[i].Time)
				{
					const FCurveKey& A = Keys[i - 1];
					const FCurveKey& B = Keys[i];

					const float Alpha = (InTime - A.Time) / (B.Time - A.Time);
					return FMath::Lerp(A.Value, B.Value, Alpha);
				}
			}

			return 0.f; // Should not reach here
		}

		bool Serialize(FArchive& Ar)
		{
			Ar << Keys;
			return true;
		}

		friend FArchive& operator<<(FArchive& Ar, FLinearCurve& Curve)
		{
			Curve.Serialize(Ar);
			return Ar;
		}
	};

} // namespace Chaos