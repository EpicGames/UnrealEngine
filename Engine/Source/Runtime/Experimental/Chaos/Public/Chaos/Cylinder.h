// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/ImplicitObject.h"
#include "Chaos/Plane.h"
#include "Chaos/Rotation.h"
#include "Chaos/Sphere.h"
#include "ChaosArchive.h"

namespace Chaos
{
	struct FCylinderSpecializeSamplingHelper;

	class FCylinder final : public FImplicitObject
	{
	public:
		using FImplicitObject::SignedDistance;
		using FImplicitObject::GetTypeName;

		FCylinder(const FVec3& x1, const FVec3& x2, const FReal Radius)
		    : FImplicitObject(EImplicitObject::FiniteConvex, ImplicitObjectType::Cylinder)
		    , MPlane1(x1, (x2 - x1).GetSafeNormal()) // Plane normals point inward
		    , MPlane2(x2, -MPlane1.Normal())
		    , MHeight((x2 - x1).Size())
		    , MRadius(Radius)
		    , MLocalBoundingBox(x1, x1)
		{
			MLocalBoundingBox.GrowToInclude(x2);
			MLocalBoundingBox = FAABB3(MLocalBoundingBox.Min() - FVec3(MRadius), MLocalBoundingBox.Max() + FVec3(MRadius));
		}
		FCylinder(const FCylinder& Other)
		    : FImplicitObject(EImplicitObject::FiniteConvex, ImplicitObjectType::Cylinder)
		    , MPlane1(Other.MPlane1)
		    , MPlane2(Other.MPlane2)
		    , MHeight(Other.MHeight)
		    , MRadius(Other.MRadius)
		    , MLocalBoundingBox(Other.MLocalBoundingBox)
		{
		}
		FCylinder(FCylinder&& Other)
		    : FImplicitObject(EImplicitObject::FiniteConvex, ImplicitObjectType::Cylinder)
		    , MPlane1(MoveTemp(Other.MPlane1))
		    , MPlane2(MoveTemp(Other.MPlane2))
		    , MHeight(Other.MHeight)
		    , MRadius(Other.MRadius)
		    , MLocalBoundingBox(MoveTemp(Other.MLocalBoundingBox))
		{
		}
		~FCylinder() = default;

		static constexpr EImplicitObjectType StaticType() { return ImplicitObjectType::Cylinder; }

		/**
		 * Returns sample points centered about the origin.
		 *
		 * \p NumPoints specifies how many points to generate.
		 * \p IncludeEndCaps determines whether or not points are generated on the 
		 *    end caps of the cylinder.
		 */
		TArray<FVec3> ComputeLocalSamplePoints(const int32 NumPoints, const bool IncludeEndCaps = true) const;

		/** 
		 * Returns sample points centered about the origin. 
		 *
		 * \p PointsPerUnitArea specifies how many points to generate per square 
		 *    unit (cm). 0.5 would generate 1 point per 2 square cm.
		 * \p IncludeEndCaps determines whether or not points are generated on the 
		 *    end caps of the cylinder.
		 */
		TArray<FVec3> ComputeLocalSamplePoints(const FReal PointsPerUnitArea, const bool IncludeEndCaps = true, const int32 MinPoints = 0, const int32 MaxPoints = 1000) const
		{ return ComputeLocalSamplePoints(FMath::Clamp(static_cast<int32>(ceil(PointsPerUnitArea * GetArea(IncludeEndCaps))), MinPoints, MaxPoints), IncludeEndCaps); }

		/**
		 * Returns sample points at the current location of the cylinder.
		 *
		 * \p NumPoints specifies how many points to generate.
		 * \p IncludeEndCaps determines whether or not points are generated on the 
		 *    end caps of the cylinder.
		 */
		TArray<FVec3> ComputeSamplePoints(const int32 NumPoints, const bool IncludeEndCaps = true) const;

		/** 
		 * Returns sample points at the current location of the cylinder.
		 *
		 * \p PointsPerUnitArea specifies how many points to generate per square 
		 *    unit (cm). 0.5 would generate 1 point per 2 square cm.
		 * \p IncludeEndCaps determines whether or not points are generated on the 
		 *    end caps of the cylinder.
		 */
		TArray<FVec3> ComputeSamplePoints(const FReal PointsPerUnitArea, const bool IncludeEndCaps = true, const int32 MinPoints = 0, const int32 MaxPoints = 1000) const
		{ return ComputeSamplePoints(FMath::Clamp(static_cast<int32>(ceil(PointsPerUnitArea * GetArea(IncludeEndCaps))), MinPoints, MaxPoints), IncludeEndCaps); }
#if 0
		/*virtual*/ FReal SignedDistance(const FVec3& x) const /*override*/
		{
			FVec3 V = MPlane1.X() - x;
			FReal Plane1Distance = FVec3::DotProduct(V, MPlane1.Normal());
			FReal PlaneDistance = FMath::Max(Plane1Distance, -MHeight - Plane1Distance);
			FReal CylinderDistance = (V - Plane1Distance * MPlane1.Normal()).Size() - MRadius;
			return CylinderDistance > 0.0 && PlaneDistance > 0.0 ?
			    FMath::Sqrt(FMath::Square(CylinderDistance) + FMath::Square(PlaneDistance)) :
			    FMath::Max(CylinderDistance, PlaneDistance);
		}
#endif
		virtual FReal PhiWithNormal(const FVec3& x, FVec3& Normal) const override
		{
			FVec3 Normal1, Normal2;
			const FReal Distance1 = MPlane1.PhiWithNormal(x, Normal1); // positive on the cylinder side
			if (Distance1 < 0) // off end 1
			{
				check(MPlane2.PhiWithNormal(x, Normal2) > 0);
				const FVec3 v = x - FVec3(Normal1 * Distance1 + MPlane1.X());
				if (v.Size() > MRadius)
				{
					const FVec3 Corner = v.GetSafeNormal() * MRadius + MPlane1.X();
					Normal = x - Corner;
					return Normal.SafeNormalize();
				}
				else
				{
					Normal = -Normal1;
					return -Distance1;
				}
			}
			const FReal Distance2 = MPlane2.PhiWithNormal(x, Normal2);
			if (Distance2 < 0) // off end 2
			{
				check(MPlane1.PhiWithNormal(x, Normal1) > 0);
				const FVec3 v = x - FVec3(Normal2 * Distance2 + MPlane2.X());
				if (v.Size() > MRadius)
				{
					const FVec3 Corner = v.GetSafeNormal() * MRadius + MPlane2.X();
					Normal = x - Corner;
					return Normal.SafeNormalize();
				}
				else
				{
					Normal = -Normal2;
					return -Distance2;
				}
			}
			// Both distances are positive, should add to the height of the cylinder.
			check(FMath::Abs(Distance1 + Distance2 - MHeight) < UE_KINDA_SMALL_NUMBER);
			const FVec3 SideVector = (x - FVec3(Normal1 * Distance1 + MPlane1.X()));
			const FReal SideDistance = SideVector.Size() - MRadius;
			if (SideDistance < 0)
			{
				// We're inside the cylinder. If the distance to a endcap is less
				// than the SideDistance, push out the end.
				const FReal TopDistance = Distance1 < Distance2 ? Distance1 : Distance2;
				if (TopDistance < -SideDistance)
				{
					Normal = Distance1 < Distance2 ? -Normal1 : -Normal2;
					return -TopDistance;
				}
			}
			Normal = SideVector.GetSafeNormal();
			return SideDistance;
		}

		virtual const FAABB3 BoundingBox() const override { return MLocalBoundingBox; }

		virtual FReal GetRadius() const override { return MRadius; }
		FReal GetHeight() const { return MHeight; }
		const FVec3& GetX1() const { return MPlane1.X(); }
		const FVec3& GetX2() const { return MPlane2.X(); }
		/** Returns the bottommost point on the cylinder. */
		const FVec3& GetOrigin() const { return MPlane1.X(); }
		/** Returns the topmost point on the cylinder. */
		const FVec3& GetInsertion() const { return MPlane2.X(); }
		FVec3 GetCenter() const { return (MPlane1.X() + MPlane2.X()) * (FReal)0.5; }
		/** Returns the centroid (center of mass). */
		FVec3 GetCenterOfMass() const { return GetCenter(); }

		FVec3 GetAxis() const { return (MPlane2.X() - MPlane1.X()).GetSafeNormal(); }

		FReal GetArea(const bool IncludeEndCaps = true) const { return GetArea(MHeight, MRadius, IncludeEndCaps); }
		static FReal GetArea(const FReal Height, const FReal Radius, const bool IncludeEndCaps)
		{
			static const FReal PI2 = 2. * UE_PI;
			return IncludeEndCaps ?
				PI2 * Radius * (Height + Radius) :
				PI2 * Radius * Height;
		}

		FReal GetVolume() const { return GetVolume(MHeight, MRadius); }
		static FReal GetVolume(const FReal Height, const FReal Radius) { return UE_PI * Radius * Radius * Height; }

		FMatrix33 GetInertiaTensor(const FReal Mass) const { return GetInertiaTensor(Mass, MHeight, MRadius); }
		static FMatrix33 GetInertiaTensor(const FReal Mass, const FReal Height, const FReal Radius)
		{
			// https://www.wolframalpha.com/input/?i=cylinder
			const FReal RR = Radius * Radius;
			const FReal Diag12 = static_cast<FReal>(Mass / 12. * (3.*RR + Height*Height));
			const FReal Diag3 = static_cast<FReal>(Mass / 2. * RR);
			return FMatrix33(Diag12, Diag12, Diag3);
		}

		FRotation3 GetRotationOfMass() const { return GetRotationOfMass(GetAxis()); }
		static FRotation3 GetRotationOfMass(const FVec3& Axis)
		{ 
			// since the cylinder stores an axis and the InertiaTensor is assumed to be along the ZAxis
			// we need to make sure to return the rotation of the axis from Z
			return FRotation3::FromRotatedVector(FVec3(0, 0, 1), Axis);
		}

		virtual void Serialize(FChaosArchive& Ar)
		{
			FChaosArchiveScopedMemory ScopedMemory(Ar, GetTypeName());
			FImplicitObject::SerializeImp(Ar);
			Ar << MPlane1;
			Ar << MPlane2;
			Ar << MHeight;
			Ar << MRadius;
			TBox<FReal, 3>::SerializeAsAABB(Ar, MLocalBoundingBox);
		}

	private:
		virtual Pair<FVec3, bool> FindClosestIntersectionImp(const FVec3& StartPoint, const FVec3& EndPoint, const FReal Thickness) const override
		{
			TArray<Pair<FReal, FVec3>> Intersections;
			// Flatten to Plane defined by StartPoint and MPlane1.Normal()
			// Project End and Center into Plane
			FVec3 ProjectedEnd = EndPoint - FVec3::DotProduct(EndPoint - StartPoint, MPlane1.Normal()) * MPlane1.Normal();
			FVec3 ProjectedCenter = MPlane1.X() - FVec3::DotProduct(MPlane1.X() - StartPoint, MPlane1.Normal()) * MPlane1.Normal();
			auto ProjectedSphere = FSphere(ProjectedCenter, MRadius);
			auto InfiniteCylinderIntersection = ProjectedSphere.FindClosestIntersection(StartPoint, ProjectedEnd, Thickness);
			if (InfiniteCylinderIntersection.Second)
			{
				auto UnprojectedIntersection = TPlane<FReal, 3>(InfiniteCylinderIntersection.First, (StartPoint - InfiniteCylinderIntersection.First).GetSafeNormal()).FindClosestIntersection(StartPoint, EndPoint, 0);
				check(UnprojectedIntersection.Second);
				Intersections.Add(MakePair((FReal)(UnprojectedIntersection.First - StartPoint).Size(), UnprojectedIntersection.First));
			}
			auto Plane1Intersection = MPlane1.FindClosestIntersection(StartPoint, EndPoint, Thickness);
			if (Plane1Intersection.Second)
				Intersections.Add(MakePair((FReal)(Plane1Intersection.First - StartPoint).Size(), Plane1Intersection.First));
			auto Plane2Intersection = MPlane2.FindClosestIntersection(StartPoint, EndPoint, Thickness);
			if (Plane2Intersection.Second)
				Intersections.Add(MakePair((FReal)(Plane2Intersection.First - StartPoint).Size(), Plane2Intersection.First));
			Intersections.Sort([](const Pair<FReal, FVec3>& Elem1, const Pair<FReal, FVec3>& Elem2) { return Elem1.First < Elem2.First; });
			for (const auto& Elem : Intersections)
			{
				if (SignedDistance(Elem.Second) <= (Thickness + 1e-4))
				{
					return MakePair(Elem.Second, true);
				}
			}
			return MakePair(FVec3(0), false);
		}

		virtual uint32 GetTypeHash() const override
		{
			const uint32 PlaneHashes = HashCombine(MPlane1.GetTypeHash(), MPlane2.GetTypeHash());
			const uint32 PropertyHash = HashCombine(::GetTypeHash(MHeight), ::GetTypeHash(MRadius));

			return HashCombine(PlaneHashes, PropertyHash);
		}

		//needed for serialization
		FCylinder() : FImplicitObject(EImplicitObject::HasBoundingBox, ImplicitObjectType::Cylinder) {}
		friend FImplicitObject;	//needed for serialization

	private:
		TPlane<FReal, 3> MPlane1, MPlane2;
		FReal MHeight, MRadius;
		FAABB3 MLocalBoundingBox;
	};

	struct FCylinderSpecializeSamplingHelper
	{
		static FORCEINLINE void ComputeSamplePoints(TArray<FVec3>& Points, const FCylinder& Cylinder, const int32 NumPoints, const bool IncludeEndCaps = true)
		{
			if (NumPoints <= 1 || Cylinder.GetRadius() <= UE_KINDA_SMALL_NUMBER)
			{
				const int32 Offset = Points.Num();
				if (Cylinder.GetHeight() <= UE_KINDA_SMALL_NUMBER)
				{
					Points.SetNumUninitialized(Offset + 1);
					Points[Offset] = Cylinder.GetCenter();
				}
				else
				{
					Points.SetNumUninitialized(Offset + 3);
					Points[Offset + 0] = Cylinder.GetOrigin();
					Points[Offset + 1] = Cylinder.GetCenter();
					Points[Offset + 2] = Cylinder.GetInsertion();
				}
				return;
			}
			ComputeGoldenSpiralPoints(Points, Cylinder, NumPoints, IncludeEndCaps);
		}

		static FORCEINLINE void ComputeGoldenSpiralPoints(TArray<FVec3>& Points, const FCylinder& Cylinder, const int32 NumPoints, const bool IncludeEndCaps = true)
		{ ComputeGoldenSpiralPoints(Points, Cylinder.GetOrigin(), Cylinder.GetAxis(), Cylinder.GetRadius(), Cylinder.GetHeight(), NumPoints, IncludeEndCaps); }

		/**
		 * Use the golden spiral method to generate evenly spaced points on a cylinder.
		 *
		 * The "golden" part is derived from the golden ratio; stand at the center,
		 * turn a golden ratio of whole turns, then emit a point in that direction.
		 *
		 * Points are generated starting from the bottom of the cylinder, ending at 
		 * the top.  Contiguous entries in \p Points generally will not be spatially
		 * adjacent.
		 *
		 * \p Points to append to.
		 * \p Origin is the bottom-most point of the cylinder.
		 * \p Axis is the orientation of the cylinder.
		 * \p Radius is the radius of the cylinder.
		 * \p Height is the height of the cylinder.
		 * \p NumPoints is the number of points to generate.
		 * \p IncludeEndCaps determines whether or not points are generated on the 
		 *    end caps of the cylinder.
		 * \p SpiralSeed is the starting index for golden spiral generation.  When 
		 *    using this method to continue a spiral started elsewhere, \p SpiralSeed 
		 *    should equal the number of particles already created.
		 */
		static FORCEINLINE void ComputeGoldenSpiralPoints(
		    TArray<FVec3>& Points,
		    const FVec3& Origin,
		    const FVec3& Axis,
		    const FReal Radius,
		    const FReal Height,
		    const int32 NumPoints,
		    const bool IncludeEndCaps = true,
		    int32 SpiralSeed = 0)
		{
			// Axis should be normalized.
			checkSlow(FMath::Abs(Axis.Size() - 1.0) < UE_KINDA_SMALL_NUMBER);

			const int32 Offset = Points.Num();
			ComputeGoldenSpiralPointsUnoriented(Points, Radius, Height, NumPoints, IncludeEndCaps, SpiralSeed);

			// At this point, Points are centered about the origin (0,0,0), built
			// along the Z axis.  Transform them to where they should be.
			const FReal HalfHeight = Height / 2;
			const FRotation3 Rotation = FRotation3::FromRotatedVector(FVec3(0, 0, 1), Axis);
			checkSlow(((Origin + Axis * Height) - (Rotation.RotateVector(FVec3(0, 0, Height)) + Origin)).Size() < UE_KINDA_SMALL_NUMBER);
			for (int32 i = Offset; i < Points.Num(); i++)
			{
				FVec3& Point = Points[i];
				const FVec3 PointNew = Rotation.RotateVector(Point + FVec3(0, 0, HalfHeight)) + Origin;
				checkSlow(FMath::Abs(FCylinder(Origin, Origin + Axis * Height, Radius).SignedDistance(PointNew)) < UE_KINDA_SMALL_NUMBER);
				Point = PointNew;
			}
		}

		/**
		 * Generates evenly spaced points on a cylinder, oriented about the Z axis, 
		 * varying from [-Height/2, Height/2].
		 *
		 * The "golden" part is derived from the golden ratio; stand at the center,
		 * turn a golden ratio of whole turns, then emit a point in that direction.
		 *
		 * Points are generated starting from the bottom of the cylinder, ending at 
		 * the top.  Contiguous entries in \p Points generally will not be spatially
		 * adjacent.
		 *
		 * \p Points to append to.
		 * \p Radius is the radius of the cylinder.
		 * \p Height is the height of the cylinder.
		 * \p NumPoints is the number of points to generate.
		 * \p IncludeEndCaps determines whether or not points are generated on the 
		 *    end caps of the cylinder.
		 * \p SpiralSeed is the starting index for golden spiral generation.  When 
		 *    using this method to continue a spiral started elsewhere, \p SpiralSeed 
		 *    should equal the number of particles already created.
		 */
		static FORCEINLINE void ComputeGoldenSpiralPointsUnoriented(
		    TArray<FVec3>& Points,
		    const FReal Radius,
		    const FReal Height,
		    const int32 NumPoints,
		    const bool IncludeEndCaps = true,
		    int32 SpiralSeed = 0)
		{
			// Evenly distribute points between the cylinder body and the end caps.
			int32 NumPointsEndCap;
			int32 NumPointsCylinder;
			if (IncludeEndCaps)
			{
				const FReal CapArea = UE_PI * Radius * Radius;
				const FReal CylArea = static_cast<FReal>(2.0 * UE_PI * Radius * Height);
				const FReal AllArea = CylArea + CapArea * 2;
				if (AllArea > UE_KINDA_SMALL_NUMBER)
				{
					NumPointsCylinder = static_cast<int32>(round(CylArea / AllArea * (FReal)NumPoints));
					NumPointsCylinder += (NumPoints - NumPointsCylinder) % 2;
					NumPointsEndCap = (NumPoints - NumPointsCylinder) / 2;
				}
				else
				{
					NumPointsCylinder = 0;
					NumPointsEndCap = (NumPoints - (NumPoints % 2)) / 2;
				}
			}
			else
			{
				NumPointsCylinder = NumPoints;
				NumPointsEndCap = 0;
			}
			const int32 NumPointsToAdd = NumPointsCylinder + NumPointsEndCap * 2;
			Points.Reserve(Points.Num() + NumPointsToAdd);

			int32 Offset = Points.Num();
			const FReal HalfHeight = Height / 2;
			TArray<FVec2> Points2D;
			Points2D.Reserve(NumPointsEndCap);
			if (IncludeEndCaps)
			{
				TSphereSpecializeSamplingHelper<FReal, 2>::ComputeGoldenSpiralPoints(
				    Points2D, FVec2((FReal)0.0), Radius, NumPointsEndCap, SpiralSeed);
				Offset = Points.AddUninitialized(Points2D.Num());
				for (int32 i = 0; i < Points2D.Num(); i++)
				{
					const FVec2& Pt = Points2D[i];
					checkSlow(Pt.Size() < Radius + UE_KINDA_SMALL_NUMBER);
					Points[i + Offset] = FVec3(Pt[0], Pt[1], -HalfHeight);
				}
				// Advance the SpiralSeed by the number of points generated.
				SpiralSeed += Points2D.Num();
			}

			Offset = Points.AddUninitialized(NumPointsCylinder);
			static const FReal Increment = static_cast<FReal>(UE_PI * (1.0 + sqrt(5)));
			for (int32 i = 0; i < NumPointsCylinder; i++)
			{
				// In the 2D sphere (disc) case, we vary R so it increases monotonically,
				// which spreads points out across the disc:
				//     const T R = FMath::Sqrt((0.5 + Index) / NumPoints) * Radius;
				// But we're mapping to a cylinder, which means we want to keep R constant.
				const FReal R = Radius;
				const FReal Theta = Increment * (0.5f + (FReal)i + (FReal)SpiralSeed);

				// Map polar coordinates to Cartesian, and vary Z by [-HalfHeight, HalfHeight].
				const FReal Z = FMath::LerpStable(-HalfHeight, HalfHeight, static_cast<FReal>(i) / (static_cast<FReal>(NumPointsCylinder) - 1));
				Points[i + Offset] =
				    FVec3(
				        R * FMath::Cos(Theta),
				        R * FMath::Sin(Theta),
				        Z);

				checkSlow(FMath::Abs(FVec2(Points[i + Offset][0], Points[i + Offset][1]).Size() - Radius) < UE_KINDA_SMALL_NUMBER);
			}
			// Advance the SpiralSeed by the number of points generated.
			SpiralSeed += NumPointsCylinder;

			if (IncludeEndCaps)
			{
				Points2D.Reset();
				TSphereSpecializeSamplingHelper<FReal, 2>::ComputeGoldenSpiralPoints(
				    Points2D, FVec2((FReal)0.0), Radius, NumPointsEndCap, SpiralSeed);
				Offset = Points.AddUninitialized(Points2D.Num());
				for (int32 i = 0; i < Points2D.Num(); i++)
				{
					const FVec2& Pt = Points2D[i];
					checkSlow(Pt.Size() < Radius + UE_KINDA_SMALL_NUMBER);
					Points[i + Offset] = FVec3(Pt[0], Pt[1], HalfHeight);
				}
			}
		}
	};

	FORCEINLINE TArray<FVec3> FCylinder::ComputeSamplePoints(const int32 NumPoints, const bool IncludeEndCaps) const
	{
		TArray<FVec3> Points;
		FCylinderSpecializeSamplingHelper::ComputeSamplePoints(Points, *this, NumPoints, IncludeEndCaps);
		return Points;
	}

	FORCEINLINE TArray<FVec3> FCylinder::ComputeLocalSamplePoints(const int32 NumPoints, const bool IncludeEndCaps) const
	{
		TArray<FVec3> Points;
		const FVec3 Mid = GetCenter();
		FCylinderSpecializeSamplingHelper::ComputeSamplePoints(
			Points, FCylinder(MPlane1.X() - Mid, MPlane2.X() - Mid, GetRadius()), NumPoints, IncludeEndCaps);
		return Points;
	}


	template<class T>
	using TCylinder UE_DEPRECATED(4.27, "Deprecated. this class is to be deleted, use FCylinder instead") = FCylinder;

	template<class T>
	using TCylinderSpecializeSamplingHelper UE_DEPRECATED(4.27, "Deprecated. this class is to be deleted, use FCylinderSpecializeSamplingHelper instead") = FCylinderSpecializeSamplingHelper;

} // namespace Chaos
