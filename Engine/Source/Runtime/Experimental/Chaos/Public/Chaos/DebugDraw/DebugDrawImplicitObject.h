// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/Declares.h"
#include "Chaos/ImplicitFwd.h"
#include "ChaosDebugDraw/ChaosDDTypes.h"
#include "Math/MathFwd.h"

#define UE_API CHAOS_API

#if CHAOS_DEBUG_DRAW

namespace Chaos
{
	struct FChaosDDImplicitObject
	{
		static UE_API void Draw(
			const FConstImplicitObjectPtr& Implicit,
			const FRigidTransform3& Transform, 
			const FColor& Color, 
			float LineThickness, 
			float Duration);
	};
}

namespace Chaos::Private
{
	void ChaosDDRenderImplicitObject(
		ChaosDD::Private::IChaosDDRenderer& Renderer, 
		const FConstImplicitObjectPtr& Implicit, 
		const FRigidTransform3& Transform,
		const FColor& Color,
		float LineThickness, 
		float Duration);
}

#endif

#undef UE_API
