// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/Declares.h"
#include "Chaos/ParticleHandleFwd.h"
#include "Chaos/ShapeInstanceFwd.h"

#define UE_API CHAOS_API

#if CHAOS_DEBUG_DRAW

namespace Chaos
{
	struct FChaosDDParticle
	{
		// Draw a particle's shapes with auto-coloring. SpaceTransform is simulation space to world transform (for RBAN)
		static UE_API void DrawShapes(
			const FRigidTransform3& SpaceTransform,
			const FGeometryParticleHandle* Particle);

		// Draw a particle's shapes with auto-coloring
		static UE_API void DrawShapes(
			const FGeometryParticleHandle* Particle);

		// Draw a particle's shapes with manual coloring
		static UE_API void DrawShapes(
			const FGeometryParticleHandle* Particle,
			const FColor& Color);

		// Draw a particle's optimized shapes with auto-coloring if it has any
		static UE_API bool DrawOptimizedShapes(
			const FGeometryParticleHandle* Particle);
	};
}

#endif

#undef UE_API
