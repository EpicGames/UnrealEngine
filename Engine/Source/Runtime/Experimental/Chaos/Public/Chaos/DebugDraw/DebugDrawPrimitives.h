// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/Declares.h"
#include "Math/MathFwd.h"

#define UE_API CHAOS_API

#if CHAOS_DEBUG_DRAW

namespace Chaos
{
	// A set of debug draw primitives.
	// 
	// NOTE: these are convenient for quick temporary debug draw but it is usually better to
	// create a custom debug draw command with its own data and generating the primitives
	// at render time, rather than capture time. Deferring the primitive generation to render
	// time means that decisions about color, line width etc. can be deferred which is useful
	// for rewinding and changing how objects are displayed.
	//
	// A debug draw command is a deferred lambda that captures the data required for 
	// rendering and uses the IChaosDDRenderer interface to perform rendering. As the command
	// is deferred, make sure all lambda parameters are captured by value, not reference!
	// By convention the Draw function is a static member of a struct, but this is not required.
	//
	// See the DrawLine	implementation for a trivial example. For a less-trivial example 
	// see FChaosDDCollisionConstraint, or for a complex example see FChaosDDParticle which 
	// supports capture-time and various render-time coloring modes.
	//
	struct FChaosDDPrimitives
	{
		static UE_API void DrawPoint(const FVector3d& Position, const FColor& Color, float PointSize, float Duration);
		static UE_API void DrawLine(const FVector3d& A, const FVector3d& B, const FColor& Color, float LineThickness, float Duration);
		static UE_API void DrawArrow(const FVector3d& A, const FVector3d& B, float ArrowSize, const FColor& Color, float LineThickness, float Duration);
		static UE_API void DrawCircle(const FVector3d& Center, const FMatrix& Axes, float Radius, const FColor& Color, float LineThickness, float Duration);
		static UE_API void DrawSphere(const FVector3d& Center, const float Radius, const FColor& Color, float LineThickness, float Duration);
		static UE_API void DrawCapsule(const FVector3d& Center, const FQuat4d& Rotation, float HalfHeight, float Radius, const FColor& Color, float LineThickness, float Duration);
		static UE_API void DrawBox(const FVector3d& Center, const FQuat4d& Rotation, const FVector3d& Size, const FColor& Color, float LineThickness, float Duration);
		static UE_API void DrawTriangle(const FVector3d& A, const FVector3d& B, const FVector3d& C, const FColor& Color, float LineThickness, float Duration);
		static UE_API void DrawString(const FVector3d& TextLocation, const FString& Text, const FColor& Color, float FontScale, bool bDrawShadow, float Duration);
	};
}

#endif

#undef UE_API
