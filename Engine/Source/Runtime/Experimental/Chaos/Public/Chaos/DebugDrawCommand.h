// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/ChaosDebugDrawDeclares.h"
#include "Containers/List.h"
#include "Math/Vector.h"
#include "Math/Color.h"
#include "Math/Quat.h"

class AActor;

#if CHAOS_DEBUG_DRAW

namespace Chaos
{
	struct FLatentDrawCommand
	{
		FVector LineStart;
		FVector LineEnd;
		FColor Color;
		int32 Segments;
		bool bPersistentLines;
		float ArrowSize;
		float LifeTime;
		uint8 DepthPriority;
		float Thickness;
		FReal Radius;
		FReal HalfHeight;
		FVector Center;
		FVector Extent;
		FQuat Rotation;
		FVector TextLocation;
		FString Text;
		class AActor* TestBaseActor;
		bool bDrawShadow;
		float FontScale;
		float Duration;
		FMatrix TransformMatrix;
		bool bDrawAxis;
		FVector YAxis;
		FVector ZAxis;

		enum class EDrawType
		{
			Point,
			Line,
			DirectionalArrow,
			Sphere,
			Box,
			String,
			Circle,
			Capsule,
		} Type;

		FLatentDrawCommand()
			: TestBaseActor(nullptr)
		{
		}

		static FLatentDrawCommand DrawPoint(const FVector& Position, const FColor& Color, bool bPersistentLines, float LifeTime, uint8 DepthPriority, float Thickness)
		{
			FLatentDrawCommand Command;
			Command.LineStart = Position;
			Command.Color = Color;
			Command.bPersistentLines = bPersistentLines;
			Command.LifeTime = LifeTime;
			Command.DepthPriority = DepthPriority;
			Command.Thickness = Thickness;
			Command.Type = EDrawType::Point;
			return Command;
		}


		static FLatentDrawCommand DrawLine(const FVector& LineStart, const FVector& LineEnd, const FColor& Color, bool bPersistentLines, float LifeTime, uint8 DepthPriority, float Thickness)
		{
			FLatentDrawCommand Command;
			Command.LineStart = LineStart;
			Command.LineEnd = LineEnd;
			Command.Color = Color;
			Command.bPersistentLines = bPersistentLines;
			Command.LifeTime = LifeTime;
			Command.DepthPriority = DepthPriority;
			Command.Thickness = Thickness;
			Command.Type = EDrawType::Line;
			return Command;
		}

		static FLatentDrawCommand DrawDirectionalArrow(const FVector& LineStart, FVector const& LineEnd, float ArrowSize, const FColor& Color, bool bPersistentLines, float LifeTime, uint8 DepthPriority, float Thickness)
		{
			FLatentDrawCommand Command;
			Command.LineStart = LineStart;
			Command.LineEnd = LineEnd;
			Command.ArrowSize = ArrowSize;
			Command.Color = Color;
			Command.bPersistentLines = bPersistentLines;
			Command.LifeTime = LifeTime;
			Command.DepthPriority = DepthPriority;
			Command.Thickness = Thickness;
			Command.Type = EDrawType::DirectionalArrow;
			return Command;
		}

		static FLatentDrawCommand DrawDebugSphere(const FVector& Center, FVector::FReal Radius, int32 Segments, const FColor& Color, bool bPersistentLines, float LifeTime, uint8 DepthPriority, float Thickness)
		{
			FLatentDrawCommand Command;
			Command.LineStart = Center;
			Command.Radius = Radius;
			Command.Color = Color;
			Command.Segments = Segments;
			Command.bPersistentLines = bPersistentLines;
			Command.LifeTime = LifeTime;
			Command.DepthPriority = DepthPriority;
			Command.Thickness = Thickness;
			Command.Type = EDrawType::Sphere;
			return Command;
		}

		static FLatentDrawCommand DrawDebugBox(const FVector& Center, const FVector& Extent, const FQuat& Rotation, const FColor& Color, bool bPersistentLines, float LifeTime, uint8 DepthPriority, float Thickness)
		{
			FLatentDrawCommand Command;
			Command.Center = Center;
			Command.Extent = Extent;
			Command.Rotation = Rotation;
			Command.Color = Color;
			Command.bPersistentLines = bPersistentLines;
			Command.LifeTime = LifeTime;
			Command.DepthPriority = DepthPriority;
			Command.Thickness = Thickness;
			Command.Type = EDrawType::Box;
			return Command;
		}

		static FLatentDrawCommand DrawDebugString(const FVector& TextLocation, const FString& Text, class AActor* TestBaseActor, const FColor& Color, float Duration, bool bDrawShadow, float FontScale)
		{
			FLatentDrawCommand Command;
			Command.TextLocation = TextLocation;
			Command.Text = Text;
			Command.TestBaseActor = TestBaseActor;
			Command.Color = Color;
			Command.Duration = Duration;
			Command.LifeTime = Duration;
			Command.bDrawShadow = bDrawShadow;
			Command.FontScale = FontScale;
			Command.Type = EDrawType::String;
			return Command;
		}

		static FLatentDrawCommand DrawDebugCircle(const FVector& Center, FReal Radius, int32 Segments, const FColor& Color, bool bPersistentLines, float LifeTime, uint8 DepthPriority, float Thickness, const FVector& YAxis, const FVector& ZAxis, bool bDrawAxis)
		{
			FLatentDrawCommand Command;
			Command.Center = Center;
			Command.Radius = Radius;
			Command.Segments = Segments;
			Command.Color = Color;
			Command.bPersistentLines = bPersistentLines;
			Command.LifeTime = LifeTime;
			Command.DepthPriority = DepthPriority;
			Command.Thickness = Thickness;
			Command.YAxis = YAxis;
			Command.ZAxis = ZAxis;
			Command.bDrawAxis = bDrawAxis;
			Command.Type = EDrawType::Circle;
			return Command;
		}

		static FLatentDrawCommand DrawDebugCapsule(const FVector& Center, FReal HalfHeight, FReal Radius, const FQuat& Rotation, const FColor& Color, bool bPersistentLines, float LifeTime, uint8 DepthPriority, float Thickness)
		{
			FLatentDrawCommand Command;
			Command.Center = Center;
			Command.HalfHeight = HalfHeight;
			Command.Radius = Radius;
			Command.Rotation = Rotation;
			Command.Color = Color;
			Command.bPersistentLines = bPersistentLines;
			Command.LifeTime = LifeTime;
			Command.DepthPriority = DepthPriority;
			Command.Thickness = Thickness;
			Command.Type = EDrawType::Capsule;
			return Command;
		}
	};
}

#endif