// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/ChaosDebugDrawDeclares.h"
#include "Chaos/AABB.h"
#include "Chaos/DebugDrawCommand.h"
#include "Containers/List.h"
#include "Math/Vector.h"
#include "Math/Color.h"
#include "Math/Quat.h"
#include "Misc/ScopeLock.h"
#include "HAL/ThreadSafeBool.h"
#include "HAL/IConsoleManager.h"

#include "ChaosDebugDraw/ChaosDDContext.h"
#include "ChaosDebugDraw/ChaosDDFrame.h"

class AActor;


#if CHAOS_DEBUG_DRAW
namespace Chaos
{
extern CHAOS_API bool bChaosDebugDraw_UseNewQueue;
extern CHAOS_API bool bChaosDebugDraw_UseLegacyQueue;

/** Thread-safe single-linked list (lock-free). (Taken from Light Mass, should probably just move into core) */
template<typename ElementType>
class TListThreadSafe
{
public:

	/** Initialization constructor. */
	TListThreadSafe() :
		FirstElement(nullptr)
	{}

	/**
	* Adds an element to the list.
	* @param Element	Newly allocated and initialized list element to add.
	*/
	void AddElement(TList<ElementType>* Element)
	{
		// Link the element at the beginning of the list.
		TList<ElementType>* LocalFirstElement;
		do
		{
			LocalFirstElement = FirstElement;
			Element->Next = LocalFirstElement;
		} while (FPlatformAtomics::InterlockedCompareExchangePointer((void**)&FirstElement, Element, LocalFirstElement) != LocalFirstElement);
	}

	/**
	* Clears the list and returns the elements.
	* @return	List of all current elements. The original list is cleared. Make sure to delete all elements when you're done with them!
	*/
	TList<ElementType>* ExtractAll()
	{
		// Atomically read the complete list and clear the shared head pointer.
		TList<ElementType>* LocalFirstElement;
		do
		{
			LocalFirstElement = FirstElement;
		} while (FPlatformAtomics::InterlockedCompareExchangePointer((void**)&FirstElement, NULL, LocalFirstElement) != LocalFirstElement);
		return LocalFirstElement;
	}

	/**
	*	Clears the list.
	*/
	void Clear()
	{
		while (FirstElement)
		{
			// Atomically read the complete list and clear the shared head pointer.
			TList<ElementType>* Element = ExtractAll();

			// Delete all elements in the local list.
			while (Element)
			{
				TList<ElementType>* NextElement = Element->Next;
				delete Element;
				Element = NextElement;
			};
		};
	}

private:

	TList<ElementType>* FirstElement;
};

/** A thread safe way to generate latent debug drawing. (This is picked up later by the geometry collection component which is a total hack for now, but needed to get into an engine world ) */
class FDebugDrawQueue
{
public:
	enum EBuffer
	{
		Internal = 0,
		External = 1
	};

	void ExtractAllElements(TArray<FLatentDrawCommand>& OutDrawCommands, bool bPaused = false)
	{
		FScopeLock Lock(&CommandQueueCS);

		// Move the buffer into the output, and reserve space in the new buffer to prevent excess allocations and copies
		int Capacity = CommandQueue.Max();
		OutDrawCommands = MoveTemp(CommandQueue);
		CommandQueue.Reserve(Capacity);
		RequestedCommandCost = 0;
	}

	void DrawDebugPoint(const FVector& Position, const FColor& Color, bool bPersistentLines, float LifeTime, uint8 DepthPriority, float Thickness)
	{
		if (IsDebugDrawingEnabled())
		{
			FScopeLock Lock(&CommandQueueCS);
			if (AcceptCommand(1, Position))
			{
				AddCommand(FLatentDrawCommand::DrawPoint(Position, Color, bPersistentLines, LifeTime, DepthPriority, Thickness));
			}
		}
	}

	void DrawDebugLine(const FVector& LineStart, const FVector& LineEnd, const FColor& Color, bool bPersistentLines = false, float LifeTime = -1.f, uint8 DepthPriority = 0, float Thickness = 0.f)
	{
		if (IsDebugDrawingEnabled())
		{
			FScopeLock Lock(&CommandQueueCS);
			if (AcceptCommand(1, LineStart))
			{
				AddCommand(FLatentDrawCommand::DrawLine(LineStart, LineEnd, Color, bPersistentLines, LifeTime, DepthPriority, Thickness));
			}
		}
	}

	void DrawDebugDirectionalArrow(const FVector& LineStart, const FVector& LineEnd, float ArrowSize, const FColor& Color, bool bPersistentLines = false, float LifeTime = -1.f, uint8 DepthPriority = 0, float Thickness = 0.f)
	{
		if (IsDebugDrawingEnabled())
		{
			FScopeLock Lock(&CommandQueueCS);
			if (AcceptCommand(3, LineStart))
			{
				AddCommand(FLatentDrawCommand::DrawDirectionalArrow(LineStart, LineEnd, ArrowSize, Color, bPersistentLines, LifeTime, DepthPriority, Thickness));
			}
		}
	}

	void DrawDebugCoordinateSystem(const FVector& Position, const FRotator& AxisRot, FReal Scale, bool bPersistentLines = false, float LifeTime = -1.f, uint8 DepthPriority = 0, float Thickness = 0.f)
	{
		if (IsDebugDrawingEnabled())
		{
			FScopeLock Lock(&CommandQueueCS);

			FRotationMatrix R(AxisRot);
			FVector const X = R.GetScaledAxis(EAxis::X);
			FVector const Y = R.GetScaledAxis(EAxis::Y);
			FVector const Z = R.GetScaledAxis(EAxis::Z);

			if (AcceptCommand(3, Position))
			{
				AddCommand(FLatentDrawCommand::DrawLine(Position, Position + X * Scale, FColor::Red, bPersistentLines, LifeTime, DepthPriority, Thickness));
				AddCommand(FLatentDrawCommand::DrawLine(Position, Position + Y * Scale, FColor::Green, bPersistentLines, LifeTime, DepthPriority, Thickness));
				AddCommand(FLatentDrawCommand::DrawLine(Position, Position + Z * Scale, FColor::Blue, bPersistentLines, LifeTime, DepthPriority, Thickness));
			}
		}
	}


	void DrawDebugSphere(FVector const& Center, FReal Radius, int32 Segments, const FColor& Color, bool bPersistentLines = false, float LifeTime = -1.f, uint8 DepthPriority = 0, float Thickness = 0.f)
	{
		if (IsDebugDrawingEnabled())
		{
			FScopeLock Lock(&CommandQueueCS);
			const int Cost = Segments * Segments;
			if (AcceptCommand(Cost, Center))
			{
				AddCommand(FLatentDrawCommand::DrawDebugSphere(Center, Radius, Segments, Color, bPersistentLines, LifeTime, DepthPriority, Thickness));
			}
		}
	}

	void DrawDebugBox(FVector const& Center, FVector const& Extent, const FQuat& Rotation, FColor const& Color, bool bPersistentLines, float LifeTime, uint8 DepthPriority, float Thickness)
	{
		if (IsDebugDrawingEnabled())
		{
			FScopeLock Lock(&CommandQueueCS);
			if (AcceptCommand(12, Center))
			{
				AddCommand(FLatentDrawCommand::DrawDebugBox(Center, Extent, Rotation, Color, bPersistentLines, LifeTime, DepthPriority, Thickness));
			}
		}
	}

	void DrawDebugString(FVector const& TextLocation, const FString& Text, class AActor* TestBaseActor, FColor const& Color, float Duration, bool bDrawShadow, float FontScale)
	{
		if (IsDebugDrawingEnabled())
		{
			FScopeLock Lock(&CommandQueueCS);
			int Cost = Text.Len();
			if (AcceptCommand(Cost, TextLocation))
			{
				AddCommand(FLatentDrawCommand::DrawDebugString(TextLocation, Text, TestBaseActor, Color, Duration, bDrawShadow, FontScale));
			}
		}
	}

	void DrawDebugCircle(const FVector& Center, FReal Radius, int32 Segments, const FColor& Color, bool bPersistentLines, float LifeTime, uint8 DepthPriority, float Thickness, const FVector& YAxis, const FVector& ZAxis, bool bDrawAxis)
	{
		if (IsDebugDrawingEnabled())
		{
			FScopeLock Lock(&CommandQueueCS);
			if (AcceptCommand(Segments, Center))
			{
				AddCommand(FLatentDrawCommand::DrawDebugCircle(Center, Radius, Segments, Color, bPersistentLines, LifeTime, DepthPriority, Thickness, YAxis, ZAxis, bDrawAxis));
			}
		}
	}

	void DrawDebugCapsule(const FVector& Center, FReal HalfHeight, FReal Radius, const FQuat& Rotation, const FColor& Color, bool bPersistentLines, float LifeTime, uint8 DepthPriority, float Thickness)
	{
		if (IsDebugDrawingEnabled())
		{
			FScopeLock Lock(&CommandQueueCS);
			if (AcceptCommand(16, Center))
			{
				AddCommand(FLatentDrawCommand::DrawDebugCapsule(Center, HalfHeight, Radius, Rotation, Color, bPersistentLines, LifeTime, DepthPriority, Thickness));
			}
		}
	}

	void SetEnabled(bool bInEnabled)
	{
		FScopeLock Lock(&CommandQueueCS);
		bEnableDebugDrawing = bInEnabled;
	}

	void SetMaxCost(int32 InMaxCost)
	{
		FScopeLock Lock(&CommandQueueCS);
		MaxCommandCost = InMaxCost;
	}

	void SetRegionOfInterest(const FVector& Pos, float InRadius)
	{
		FScopeLock Lock(&CommandQueueCS);
		CenterOfInterest = Pos;
		RadiusOfInterest = InRadius;
	}

	FVector GetCenterOfInterest() const
	{
		return GetCenterOfInterestImpl();
	}

	FReal GetRadiusOfInterest() const
	{
		return GetRadiusOfInterestImpl();
	}

	bool IsInRegionOfInterest(FVector Pos) const
	{
		return IsInRegionOfInterestImpl(Pos);
	}

	bool IsInRegionOfInterest(FVector Pos, FReal Radius) const
	{
		return IsInRegionOfInterestImpl(Pos, Radius);
	}

	bool IsInRegionOfInterest(FAABB3 Bounds) const
	{
		return IsInRegionOfInterestImpl(Bounds);
	}

	CHAOS_API void SetConsumerActive(void* Consumer, bool bConsumerActive);

	static CHAOS_API FDebugDrawQueue& GetInstance();

	static bool IsDebugDrawingEnabled()
	{
		// Don't queue up debug draws unless something is consuming the queue, otherwise it can build up forever
		return GetInstance().bEnableDebugDrawing && (GetInstance().NumConsumers > 0);
	}

private:
	FDebugDrawQueue()
		: RequestedCommandCost(0)
		, MaxCommandCost(10000)
		, CenterOfInterest(FVector::ZeroVector)
		, RadiusOfInterest(0)
		, bEnableDebugDrawing(false)
	{}
	~FDebugDrawQueue() = default;

	FVector GetCenterOfInterestImpl() const
	{
		if (bChaosDebugDraw_UseNewQueue)
		{
			ChaosDD::Private::FChaosDDFrameWriter DDWriter = ChaosDD::Private::FChaosDDContext::Get().GetWriter();
			return DDWriter.GetDrawRegion().Center;
		}

		return CenterOfInterest;
	}

	FReal GetRadiusOfInterestImpl() const
	{
		if (bChaosDebugDraw_UseNewQueue)
		{
			ChaosDD::Private::FChaosDDFrameWriter DDWriter = ChaosDD::Private::FChaosDDContext::Get().GetWriter();
			return DDWriter.GetDrawRegion().W;
		}

		return RadiusOfInterest;
	}

	bool IsInRegionOfInterestImpl(FVector Pos) const
	{
		if (bChaosDebugDraw_UseNewQueue)
		{
			ChaosDD::Private::FChaosDDFrameWriter DDWriter = ChaosDD::Private::FChaosDDContext::Get().GetWriter();
			return DDWriter.IsInDrawRegion(Pos);
		}

		return (RadiusOfInterest <= 0.0f) || ((Pos - CenterOfInterest).SizeSquared() < RadiusOfInterest * RadiusOfInterest);
	}

	bool IsInRegionOfInterestImpl(FVector Pos, FReal Radius) const
	{
		if (bChaosDebugDraw_UseNewQueue)
		{
			ChaosDD::Private::FChaosDDFrameWriter DDWriter = ChaosDD::Private::FChaosDDContext::Get().GetWriter();
			return DDWriter.IsInDrawRegion(FSphere3d(Pos, Radius));
		}

		return (RadiusOfInterest <= 0.0f) || ((Pos - CenterOfInterest).SizeSquared() < (RadiusOfInterest + Radius) * (RadiusOfInterest + Radius));
	}

	bool IsInRegionOfInterestImpl(FAABB3 Bounds) const
	{
		if (bChaosDebugDraw_UseNewQueue)
		{
			ChaosDD::Private::FChaosDDFrameWriter DDWriter = ChaosDD::Private::FChaosDDContext::Get().GetWriter();
			return DDWriter.IsInDrawRegion(FBox3d(Bounds.Min(), Bounds.Max()));
		}
		return Bounds.ThickenSymmetrically(FVec3(RadiusOfInterest)).Contains(CenterOfInterest);
	}

	bool AcceptCommand(int Cost, const FVec3& Position)
	{
		if (bChaosDebugDraw_UseNewQueue)
		{
			ChaosDD::Private::FChaosDDFrameWriter DDWriter = ChaosDD::Private::FChaosDDContext::Get().GetWriter();
			if (DDWriter.IsInDrawRegion(Position))
			{
				return DDWriter.AddToCost(Cost);
			}
			return false;
		}

		// If we get here...
		// This thread has not been set up for the new debug draw system so default back to the old
		// way that has issues with timing in async mode.
		if (bChaosDebugDraw_UseLegacyQueue)
		{
			if (IsInRegionOfInterest(Position))
			{
				RequestedCommandCost += Cost;
				return IsInBudget();
			}
		}

		return false;
	}

	void AddCommand(const FLatentDrawCommand& Command)
	{
		// Try to add to the new debug draw system which queues commands per World and ticking thread
		if (bChaosDebugDraw_UseNewQueue)
		{
			ChaosDD::Private::FChaosDDContext::GetWriter().EnqueueLatentCommand(Command);
			return;
		}

		// If we get here...
		// This thread has not been set up for the new debug draw system so default back to the old
		// way that has issues with timing in async mode.
		if (bChaosDebugDraw_UseLegacyQueue)
		{
			CommandQueue.Add(Command);
		}
	}

	bool IsInBudget() const
	{
		return (MaxCommandCost <= 0) || (RequestedCommandCost <= MaxCommandCost);
	}

	TArray<FLatentDrawCommand> CommandQueue;
	int32 RequestedCommandCost;
	int32 MaxCommandCost;
	FCriticalSection CommandQueueCS;

	TArray<void*> Consumers;
	int32 NumConsumers;
	FCriticalSection ConsumersCS;

	FVector CenterOfInterest;
	FReal RadiusOfInterest;
	bool bEnableDebugDrawing;
};
}
#endif
