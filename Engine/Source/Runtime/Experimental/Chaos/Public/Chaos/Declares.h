// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/ParticleHandleFwd.h"
#include "Chaos/PBDRigidsEvolutionFwd.h"

namespace Chaos
{
	typedef FPBDRigidsSolver FPhysicsSolver;

	class FImplicitObject;
	class FPerShapeData;
	template <typename TSOA> class TParticleView;
	template<class T, int d> class TPBDRigidParticles;
}

#ifndef CHAOS_DEBUG_DRAW
// this matches UE_ENABLE_DEBUG_DRAWING, but that's in Engine, which this code can't see
#define CHAOS_DEBUG_DRAW (!(UE_BUILD_SHIPPING || UE_BUILD_TEST) || WITH_EDITOR)
#endif

#define TODO_REIMPLEMENT_EVENTS_DATA_ARRAYS (CHAOS_PARTICLEHANDLE_TODO && 1)
#define TODO_REIMPLEMENT_REMOVED_PROXY_STORAGE (CHAOS_PARTICLEHANDLE_TODO && 1)
#define TODO_REIMPLEMENT_SPATIAL_ACCELERATION_ACCESS (CHAOS_PARTICLEHANDLE_TODO && 1)
#define TODO_REIMPLEMENT_DEBUG_SUBSTEP (CHAOS_PARTICLEHANDLE_TODO && 1)
#define TODO_REIMPLEMENT_TIMESTEP_MULTIPLIER (CHAOS_PARTICLEHANDLE_TODO && 1)
#define TODO_REIMPLEMENT_SOLVER_SETTINGS_ACCESSORS (CHAOS_PARTICLEHANDLE_TODO && 1)
#define TODO_REIMPLEMENT_SOLVER_PAUSING (CHAOS_PARTICLEHANDLE_TODO && 1)
#define TODO_REIMPLEMENT_SOLVER_ENABLING (CHAOS_PARTICLEHANDLE_TODO && 1)
#define TODO_REIMPLEMENT_SERIALIZATION_FOR_PERF_TEST (CHAOS_PARTICLEHANDLE_TODO && 1)
#define TODO_REIMPLMEENT_EVOLUTION_ACCESSORS (CHAOS_PARTICLEHANDLE_TODO && 1)
#define TODO_REIMPLEMENT_DYNAMIC_CONSTRAINT_ACCESSORS (CHAOS_PARTICLEHANDLE_TODO && 1)
#define TODO_REIMPLEMENT_FIELDS_TO_USE_PARTICLEHANDLES (CHAOS_PARTICLEHANDLE_TODO && 1)
#define TODO_REIMPLEMENT_GEOMETRY_COLLECTION_PHYSICS (CHAOS_PARTICLEHANDLE_TODO && 1)
#define TODO_REIMPLEMENT_RIGID_CLUSTERING (CHAOS_PARTICLEHANDLE_TODO && 1)
#define TODO_REIMPLEMENT_GETFLOORINDEX (CHAOS_PARTICLEHANDLE_TODO && 1)
#define TODO_REIMPLEMENT_PHYSICS_PROXY_REVERSE_MAPPING (CHAOS_PARTICLEHANDLE_TODO && 1)
#define TODO_REIMPLEMENT_KINEMATIC_PROXY (CHAOS_PARTICLEHANDLE_TODO && 1)
#define TODO_REIMPLEMENT_WAKE_ISLANDS (CHAOS_PARTICLEHANDLE_TODO && 1)
#define TODO_REIMPLEMENT_GET_RIGID_PARTICLES (CHAOS_PARTICLEHANDLE_TODO && 1)
#define TODO_REIMPLEMENT_SET_PHYSICS_MATERIAL (CHAOS_PARTICLEHANDLE_TODO && 1)
#define TODO_REWRITE_ALL_CONSTRAINT_ADDS_TO_USE_HANDLES (CHAOS_PARTICLEHANDLE_TODO && 1)
#define TODO_REIMPLEMENT_GETALLCLUSTERBREAKINGS (CHAOS_PARTICLEHANDLE_TODO && 1)
#define TODO_FIX_REFERENCES_TO_ADDARRAY (CHAOS_PARTICLEHANDLE_TODO && 1)
#define TODO_REMOVE_ALL_SOLVER_CREATIONS_WHICH_OCCUR_OUTSIDE_THE_SOLVERS_MODULE (CHAOS_PARTICLEHANDLE_TODO && 1)
#define TODO_MAKE_A_BLUEPRINT_WAY_TO_ACCESS_PARTICLES (CHAOS_PARTICLEHANDLE_TODO && 1)
#define TODO_REPLACE_SOLVER_LOCK (CHAOS_PARTICLEHANDLE_TODO && 1)
#define TODO_REIMPLEMENT_INITIALIZE_EVOLUTION_FROM_PARTICLE_DATA (CHAOS_PARTICLEHANDLE_TODO && 1)
#define TODO_REIMPLEMENT_SKELETAL_TESTS (CHAOS_PARTICLEHANDLE_TODO && 1)
#define TODO_REIMPLEMENT_SCENEQUERY_CROSSENGINE (0)		// ISQAccelerator must be re-implemented for Chaos/PhysX SQ interop. Without this, GDC 2019 demos will not work as designed.

#define CHAOS_CONSTRAINTHANDLE_TODO 0
