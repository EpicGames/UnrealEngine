// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HAL/Platform.h"

#ifndef CHAOS_TEMPLATE_API
#if PLATFORM_MAC || PLATFORM_LINUX
#define CHAOS_TEMPLATE_API CHAOS_API
#else
#define CHAOS_TEMPLATE_API
#endif
#endif

#ifndef CHAOSSOLVERS_TEMPLATE_API
#if PLATFORM_MAC || PLATFORM_LINUX
#define CHAOSSOLVERS_TEMPLATE_API CHAOS_API
#else
#define CHAOSSOLVERS_TEMPLATE_API
#endif
#endif

#if COMPILE_WITHOUT_UNREAL_SUPPORT
	#include <stdint.h>
#else
	#include "CoreTypes.h"
	#include "Logging/MessageLog.h"
	#include "UObject/ExternalPhysicsCustomObjectVersion.h"
#endif
#include "Serializable.h"

#include "UObject/ExternalPhysicsCustomObjectVersion.h"
#include "UObject/FortniteValkyrieBranchObjectVersion.h"
#include "Chaos/Core.h"

#if COMPILE_WITHOUT_UNREAL_SUPPORT
#define PI 3.14159
#define check(condition)
typedef int32_t int32;
#endif

namespace Chaos
{
	// Solver uses Kg for mass, Second for time and Cm for distance
	// Since a Pascal is 1N/m2 => (Kg.m)/(m2.s2)
	// we get 1MPa = 10000 Kg/(cm.s2)
	constexpr float MegaPascalToKgPerCmS2(float MegaPascals)
	{
		return MegaPascals * 10000;
	}

	struct FChaosPhysicsMaterialStrength
	{

		// using concrete as default ( lowest values of it )
		FChaosPhysicsMaterialStrength()
			: TensileStrength(MegaPascalToKgPerCmS2(2))
			, CompressionStrength(MegaPascalToKgPerCmS2(20))
			, ShearStrength(MegaPascalToKgPerCmS2(6))
		{}

		// Unit is Kg/(cm.s2)
		float TensileStrength;
		float CompressionStrength;
		float ShearStrength;
	};

	struct FChaosPhysicsMaterialDamageModifier
	{
		/** 
		* multiplier for the geometry collection damage thresholds/ internal strain 
		* this allows for setting up unit damage threshold and use the material to scale them to the desired range of values
		*/
		float DamageThresholdMultiplier = 1.0f;
	};

	// NOTE: must match EPhysicalMaterialSoftCollisionMode
	enum class EChaosPhysicsMaterialSoftCollisionMode : uint8
	{
		None,
		RelativeThickness,
		AbsoluteThickness,
	};

	class FChaosPhysicsMaterial
	{
	public:

		/** The different combine modes to determine the friction / restitution of two actors.
		    If the two materials have two different combine modes, we use the largest one (Min beats Avg, Max beats Multiply, etc...)
			*/
		enum class ECombineMode : uint8
		{
			Avg,
			Min,
			Multiply,
			Max
		};

		FReal Friction;
		FReal StaticFriction;
		FReal Restitution;
		FReal Density;
		FReal LinearEtherDrag;
		FReal AngularEtherDrag;
		FReal SleepingLinearThreshold;
		FReal SleepingAngularThreshold;
		FReal DisabledLinearThreshold;
		FReal DisabledAngularThreshold;
		int32 SleepCounterThreshold;
		void* UserData;

		/** Settings for "sticky" friction */
		FRealSingle BaseFrictionImpulse;

		/** Variable defaults are used for \c UChaosPhysicalMaterial \c UPROPERTY defaults. */
		ECombineMode FrictionCombineMode;
		ECombineMode RestitutionCombineMode;

		FChaosPhysicsMaterialStrength Strength;
		FChaosPhysicsMaterialDamageModifier DamageModifier;

		EChaosPhysicsMaterialSoftCollisionMode SoftCollisionMode;
		FRealSingle SoftCollisionThickness;

		FChaosPhysicsMaterial()
			: Friction(0.5)
			, StaticFriction(0.0)
			, Restitution(0.1f)
			, Density(1.f)
			, LinearEtherDrag(0.0)
			, AngularEtherDrag(0.0)
			, SleepingLinearThreshold(1)
			, SleepingAngularThreshold(1)
			, DisabledLinearThreshold(0)
			, DisabledAngularThreshold(0)
			, SleepCounterThreshold(0)
			, UserData(nullptr)
			, BaseFrictionImpulse(0.0)
			, FrictionCombineMode(ECombineMode::Avg)
			, RestitutionCombineMode(ECombineMode::Avg)
			, SoftCollisionMode(EChaosPhysicsMaterialSoftCollisionMode::None)
			, SoftCollisionThickness(0)
		{
		}

		static FReal CombineHelper(FReal A, FReal B, ECombineMode Mode)
		{
			switch(Mode)
			{
			case ECombineMode::Avg: return (A+B)*(FReal)0.5;
			case ECombineMode::Min: return FMath::Min(A,B);
			case ECombineMode::Max: return FMath::Max(A,B);
			case ECombineMode::Multiply: return A*B;
			default: ensure(false);
			}

			return 0;
		}

		static ECombineMode ChooseCombineMode(ECombineMode A, ECombineMode B)
		{
			const uint8 MaxVal = FMath::Max(static_cast<uint8>(A),static_cast<uint8>(B));
			return static_cast<ECombineMode>(MaxVal);
		}

		static constexpr bool IsSerializablePtr = true;

		static void StaticSerialize(FArchive& Ar, TSerializablePtr<FChaosPhysicsMaterial>& Serializable)
		{
			FChaosPhysicsMaterial* Material = const_cast<FChaosPhysicsMaterial*>(Serializable.Get());

			if(Ar.IsLoading())
			{
				Material = new FChaosPhysicsMaterial();
				Serializable.SetFromRawLowLevel(Material);
			}

			Material->Serialize(Ar);
		}

		void Serialize(FArchive& Ar)
		{
			Ar.UsingCustomVersion(FExternalPhysicsCustomObjectVersion::GUID);

			Ar << Friction << StaticFriction << Restitution << LinearEtherDrag << AngularEtherDrag << SleepingLinearThreshold << SleepingAngularThreshold << DisabledLinearThreshold << DisabledAngularThreshold;

			if (Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) >= FExternalPhysicsCustomObjectVersion::PhysicsMaterialSleepCounterThreshold)
			{
				Ar << SleepCounterThreshold;
			}

			if (Ar.CustomVer(FFortniteValkyrieBranchObjectVersion::GUID) >= FFortniteValkyrieBranchObjectVersion::ChaosAddDensityToPhysicsMaterial)
			{
				Ar << Density;
			}
		}
	};

	class FChaosPhysicsMaterialMask
	{
	public:

		int32 SizeX;
		int32 SizeY;
		TArray<uint32> MaskData;

		int32 UVChannelIndex;
		int32 AddressX;
		int32 AddressY;

		FChaosPhysicsMaterialMask()
			: SizeX(0)
			, SizeY(0)
			, MaskData()
			, UVChannelIndex(0)
			, AddressX(0)
			, AddressY(0)
		{
		}

		static constexpr bool IsSerializablePtr = true;

		static void StaticSerialize(FArchive& Ar, TSerializablePtr<FChaosPhysicsMaterialMask>& Serializable)
		{
			FChaosPhysicsMaterialMask* MaterialMask = const_cast<FChaosPhysicsMaterialMask*>(Serializable.Get());

			if (Ar.IsLoading())
			{
				MaterialMask = new FChaosPhysicsMaterialMask();
				Serializable.SetFromRawLowLevel(MaterialMask);
			}

			MaterialMask->Serialize(Ar);
		}

		void Serialize(FArchive& Ar)
		{
			Ar << SizeX << SizeY << UVChannelIndex << AddressX << AddressY << MaskData;
		}
	};
}