// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Chaos/Deformable/ChaosDeformableSolverProxy.h"
#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"

namespace Chaos::Softs
{
	enum ERigidCollisionShapeType
	{
		Sphere,
		Box,
		Sphyl,
		Convex,
		TaperedCapsule,
		LevelSet,

		Unknown
	};

	typedef TTuple< const UObject*, ERigidCollisionShapeType, int8> FCollisionObjectKey;

	struct FCollisionObjectAddedBodies
	{
		FCollisionObjectAddedBodies( FCollisionObjectKey InKey = FCollisionObjectKey(),
			FTransform InTransform = FTransform::Identity,
			FString InType = "",
			FImplicitObject* InShapes = nullptr)
			: Key(InKey)
			, Transform(InTransform)
			, Type(InType)
			, Shapes(InShapes) {}

		FCollisionObjectKey Key = FCollisionObjectKey();
		FTransform Transform = FTransform::Identity;
		FString Type;
		FImplicitObject* Shapes = nullptr;
	};

	struct FCollisionObjectRemovedBodies
	{
		FCollisionObjectKey Key = FCollisionObjectKey();
	};

	struct FCollisionObjectUpdatedBodies
	{
		FCollisionObjectKey Key = FCollisionObjectKey();
		FTransform Transform = FTransform::Identity;
	};

	struct FCollisionObjectParticleHandel
	{
		FCollisionObjectParticleHandel(int32 InParticleIndex = INDEX_NONE,
									   int32 InActiveViewIndex = INDEX_NONE,
									   FTransform InTransform = FTransform::Identity)
			: ParticleIndex(InParticleIndex)
			, ActiveViewIndex(InActiveViewIndex)
			, Transform(InTransform) {}

		int32 ParticleIndex = INDEX_NONE;
		int32 ActiveViewIndex = INDEX_NONE;
		FTransform Transform = FTransform::Identity;
	};


	class FCollisionManagerProxy : public FThreadingProxy
	{
	public:
		typedef FThreadingProxy Super;

		FCollisionManagerProxy(UObject* InOwner)
			: Super(InOwner, TypeName())
		{}

		static FName TypeName() { return FName("CollisionManager"); }

		class FCollisionsInputBuffer : public FThreadingProxy::FBuffer
		{
			typedef FThreadingProxy::FBuffer Super;

		public:
			typedef FCollisionManagerProxy Source;

			FCollisionsInputBuffer(
				const TArray<FCollisionObjectAddedBodies>& InAdded
				, const TArray<FCollisionObjectRemovedBodies>& InRemoved
				, const TArray<FCollisionObjectUpdatedBodies>& InUpdate
				, const UObject* InOwner)
				: Super(InOwner, FCollisionManagerProxy::TypeName())
				, Added(InAdded)
				, Removed(InRemoved)
				, Updated(InUpdate)
			{}

			TArray<FCollisionObjectAddedBodies> Added;
			TArray<FCollisionObjectRemovedBodies> Removed;
			TArray<FCollisionObjectUpdatedBodies> Updated;
		};

		TArray<FCollisionObjectAddedBodies> CollisionObjectsToAdd;
		TArray< FCollisionObjectRemovedBodies > CollisionObjectsToRemove;
		TMap< FCollisionObjectKey, FCollisionObjectParticleHandel > CollisionBodies;
	};

}// namespace Chaos::Softs


