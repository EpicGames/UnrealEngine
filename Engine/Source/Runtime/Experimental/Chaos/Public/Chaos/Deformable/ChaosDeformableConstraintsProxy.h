// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Chaos/Deformable/ChaosDeformableSolverProxy.h"
#include "CoreMinimal.h"
#include "Chaos/Deformable/GaussSeidelWeakConstraints.h"
#include "UObject/ObjectMacros.h"

namespace Chaos::Softs
{
	enum EDeformableConstraintType
	{
		Kinematic,
		LinearSpring
	};

	struct FDeformableConstraintParameters
	{
		FDeformableConstraintParameters(float InStiffness = 100000.f, float InDamping = 1.f,
			EDeformableConstraintType InType = EDeformableConstraintType::Kinematic)
		: Type(InType)
		, Stiffness(InStiffness)
		, Damping(InDamping)
		{}
		EDeformableConstraintType Type;
		float Stiffness = 100000.f;
		float Damping = 1.f;
	};

	typedef TTuple< const TObjectPtr<UObject>, const TObjectPtr<UObject>, EDeformableConstraintType> FConstraintObjectKey;

	struct FConstraintObjectAdded : public FConstraintObjectKey
	{
		FConstraintObjectAdded(FConstraintObjectKey InKey = FConstraintObjectKey(),
			FDeformableConstraintParameters InParamaters = FDeformableConstraintParameters())
			: FConstraintObjectKey(InKey)
			, Parameters(InParamaters) {}

		FDeformableConstraintParameters Parameters = FDeformableConstraintParameters();
	};

	struct FConstraintObjectRemoved : public FConstraintObjectKey
	{
		FConstraintObjectRemoved(FConstraintObjectKey InKey = FConstraintObjectKey())
			: FConstraintObjectKey(InKey) {}
	};

	struct FConstraintObjectUpdated : public FConstraintObjectKey
	{
		FConstraintObjectUpdated(FConstraintObjectKey InKey = FConstraintObjectKey(),
			FDeformableConstraintParameters InParamaters = FDeformableConstraintParameters())
			: FConstraintObjectKey(InKey)
			, Parameters(InParamaters) {}

		FDeformableConstraintParameters Parameters = FDeformableConstraintParameters();
	};

	struct FConstraintObjectParticleHandel
	{
		FConstraintObjectParticleHandel(
			int32 InSourceParticleIndex = INDEX_NONE,
			int32 InTargetParticleIndex = INDEX_NONE)
			: SourceParticleIndex(InSourceParticleIndex)
			, TargetParticleIndex(InTargetParticleIndex)
		{}

		int32 SourceParticleIndex = INDEX_NONE;
		int32 TargetParticleIndex = INDEX_NONE;

		TArray<const Chaos::Softs::FGaussSeidelSpringConstraints::FGaussSeidelConstraintHandle*> Handles;
	};


	class FConstraintManagerProxy : public FThreadingProxy
	{
	public:
		typedef FThreadingProxy Super;

		FConstraintManagerProxy(UObject* InOwner)
			: Super(InOwner, TypeName())
		{}

		static FName TypeName() { return FName("ConstraintManager"); }

		class FConstraintsInputBuffer : public FThreadingProxy::FBuffer
		{
			typedef FThreadingProxy::FBuffer Super;

		public:
			typedef FConstraintManagerProxy Source;

			FConstraintsInputBuffer(
				const TArray<FConstraintObjectAdded>& InAdded
				, const TArray<FConstraintObjectRemoved>& InRemoved
				, const TArray<FConstraintObjectUpdated>& InUpdate
				, const UObject* InOwner)
				: Super(InOwner, FConstraintManagerProxy::TypeName())
				, Added(InAdded)
				, Removed(InRemoved)
				, Updated(InUpdate)
			{}

			TArray<FConstraintObjectAdded> Added;
			TArray<FConstraintObjectRemoved> Removed;
			TArray<FConstraintObjectUpdated> Updated;
		};


		
		TArray<FConstraintObjectAdded> ConstraintObjectsToAdd;
		TArray< FConstraintObjectRemoved> ConstraintObjectsToRemove;
		TMap< FConstraintObjectKey, FConstraintObjectParticleHandel > Constraints;
	};

}// namespace Chaos::Softs


