// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/Core.h"
#include "Chaos/PBDCollisionSpringConstraints.h"
#include "Chaos/PBDEvolution.h"
#include "Chaos/PBDParticles.h"
#include "Chaos/PBDSoftsSolverParticles.h"
#include "Chaos/PBDSoftsEvolutionFwd.h"
#include "Chaos/PBDTriangleMeshCollisions.h"
#include "Chaos/TriangleMesh.h"
#include "Chaos/XPBDCorotatedConstraints.h"
#include "Chaos/Deformable/GaussSeidelCorotatedConstraints.h"
#include "Chaos/Deformable/GaussSeidelNeohookeanConstraints.h"
#include "Chaos/Deformable/GaussSeidelWeakConstraints.h"
#include "Chaos/Deformable/GaussSeidelDynamicWeakConstraints.h"
#include "Chaos/Deformable/GaussSeidelSphereRepulsionConstraints.h"
#include "Chaos/Deformable/GaussSeidelUnilateralTetConstraints.h"
#include "Chaos/Deformable/GaussSeidelMainConstraint.h"
#include "Chaos/Deformable/MuscleActivationConstraints.h"
#include "Chaos/Deformable/GaussSeidelCorotatedCodimensionalConstraints.h"
#include "Chaos/Deformable/GaussSeidelLinearCodimensionalConstraints.h"
#include "Chaos/XPBDWeakConstraints.h"
#include "Chaos/BlendedXPBDCorotatedConstraints.h"
#include "Chaos/XPBDGridBasedCorotatedConstraints.h"
#include "Chaos/Deformable/ChaosDeformableSolverTypes.h"
#include "Chaos/Deformable/ChaosDeformableSolverProxy.h"
#include "Chaos/Deformable/ChaosDeformableCollisionsProxy.h"
#include "Chaos/Deformable/ChaosDeformableConstraintsProxy.h"
#include "CoreMinimal.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Templates/UniquePtr.h"

namespace Chaos::Softs
{
	class FDeformableSolver : public FPhysicsSolverEvents
	{
		friend class FGameThreadAccess;
		friend class FPhysicsThreadAccess;

	public:

		CHAOS_API FDeformableSolver(FDeformableSolverProperties InProp = FDeformableSolverProperties());
		CHAOS_API virtual ~FDeformableSolver();

		/* Physics Thread Access API */
		class FPhysicsThreadAccess
		{
		public:
			FPhysicsThreadAccess(FDeformableSolver* InSolver, const FPhysicsThreadAccessor&) : Solver(InSolver) {}
			bool operator()() { return Solver != nullptr; }

			/* Pre-Simulation Advance */
			CHAOS_API void LoadRestartData();

			/* Simulation Advance */
			CHAOS_API void UpdateProxyInputPackages();
			CHAOS_API void Simulate(FSolverReal DeltaTime);
			CHAOS_API void AdvanceDt(FSolverReal DeltaTime);
			CHAOS_API void Reset(const FDeformableSolverProperties&);
			CHAOS_API void Update(FSolverReal DeltaTime);
			CHAOS_API void UpdateOutputState(FThreadingProxy&);
			CHAOS_API TUniquePtr<FDeformablePackage> PullInputPackage();
			CHAOS_API void PushOutputPackage(int32 Frame, FDeformableDataMap&& Package);

			/* Iteration Advance */
			CHAOS_API void InitializeSimulationObjects();
			CHAOS_API void InitializeSimulationObject(FThreadingProxy&);
			CHAOS_API void InitializeKinematicConstraint();
			CHAOS_API void InitializeSelfCollisionVariables();
			CHAOS_API void RemoveSimulationObjects();

			const FDeformableSolverProperties& GetProperties() const { return Solver->GetProperties(); }

			FPBDEvolution* GetEvolution() { return Solver->Evolution.Get(); }
			const FPBDEvolution* GetEvolution() const { return Solver->Evolution.Get(); }

			TArrayCollectionArray<const UObject*>& GetObjectsMap() { return Solver->MObjects; }
			const TArrayCollectionArray<const UObject*>& GetObjectsMap() const { return Solver->MObjects; }

			/* Returns per-particle muscle activation: 0-1 for contractibles, -1 for non-contractibles */
			CHAOS_API TArray<float> GetParticleMuscleActivation();

		private:
			FDeformableSolver* Solver;
		};


		/* Game Thread Access API */
		class FGameThreadAccess
		{
		public:
			FGameThreadAccess(FDeformableSolver* InSolver, const FGameThreadAccessor&) : Solver(InSolver) {}
			bool operator()() { return Solver != nullptr; }

			int32 GetFrame() const { return Solver->GetFrame(); }
			CHAOS_API bool HasObject(UObject* InObject) const;
			CHAOS_API void AddProxy(FThreadingProxy* InObject);
			CHAOS_API void RemoveProxy(FThreadingProxy* InObject);
			CHAOS_API void PushInputPackage(int32 Frame, FDeformableDataMap&& InPackage);
			CHAOS_API void PushRestartPackage(int32 Frame, FDeformableDataMap&& InPackage);
			CHAOS_API void SetEnableSolver(bool InbEnableSolver);
			CHAOS_API bool GetEnableSolver();

			CHAOS_API TUniquePtr<FDeformablePackage> PullOutputPackage();

		private:
			FDeformableSolver* Solver;
		};

	protected:

		void SetEnableSolver(bool InbEnableSolver) {FScopeLock Lock(&SolverEnabledMutex); bEnableSolver = InbEnableSolver; }
		bool GetEnableSolver() const { return bEnableSolver; }

		/* Pre-Simulation Advance */
		CHAOS_API void LoadRestartData();
		CHAOS_API void UpdateProxyRestartPackages();
		CHAOS_API void UpdateRestartParticlePositions();

		/* Simulation Advance */
		int32 GetFrame() const { return Frame; }
		CHAOS_API void UpdateProxyInputPackages();
		CHAOS_API void Simulate(FSolverReal DeltaTime);
		CHAOS_API void AdvanceDt(FSolverReal DeltaTime);
		CHAOS_API void Reset(const FDeformableSolverProperties&);
		CHAOS_API void Update(FSolverReal DeltaTime);
		CHAOS_API void UpdateSimulationObjects(FSolverReal DeltaTime);
		CHAOS_API void UpdateOutputState(FThreadingProxy&);
		CHAOS_API void PushOutputPackage(int32 Frame, FDeformableDataMap&& Package);
		CHAOS_API TUniquePtr<FDeformablePackage> PullInputPackage();
		CHAOS_API TUniquePtr<FDeformablePackage> PullRestartPackage();

		/* Iteration Advance */
		CHAOS_API void InitializeSimulationSpace();
		CHAOS_API void InitializeSimulationObjects();
		CHAOS_API void InitializeSimulationObject(FThreadingProxy&);
		CHAOS_API void InitializeDeformableParticles(FFleshThreadingProxy&);
		CHAOS_API void UpdateTransientConstraints();
		CHAOS_API void PostProcessTransientConstraints();
		CHAOS_API void InitializeKinematicParticles(FFleshThreadingProxy&);
		CHAOS_API void InitializeTetrahedralOrTriangleConstraint(FFleshThreadingProxy&);
		CHAOS_API void InitializeGridBasedConstraints(FFleshThreadingProxy&);
		CHAOS_API void InitializeGaussSeidelConstraints(FFleshThreadingProxy& Proxy);
		CHAOS_API void InitializeWeakConstraint(FFleshThreadingProxy&);
		CHAOS_API void InitializeKinematicConstraint();
		CHAOS_API void InitializeCollisionBodies(FCollisionManagerProxy&);
		CHAOS_API void InitializeConstraintBodies(FConstraintManagerProxy& Proxy);
		CHAOS_API void InitializeSelfCollisionVariables();
		CHAOS_API void InitializeGridBasedConstraintVariables();
		CHAOS_API void InitializeGaussSeidelConstraintVariables();
		CHAOS_API void InitializeMuscleActivationVariables();
		CHAOS_API void InitializeMuscleActivation(FFleshThreadingProxy& Proxy);
		CHAOS_API void UpdateCollisionBodies(FCollisionManagerProxy&, FThreadingProxy::FKey, FSolverReal DeltaTime);
		CHAOS_API void UpdateConstraintBodies(FConstraintManagerProxy& Proxy, FThreadingProxy::FKey Owner, FSolverReal DeltaTime);
		CHAOS_API void RemoveSimulationObjects();
		CHAOS_API TArray<Chaos::TVec3<FSolverReal>> ComputeParticleTargets(const TArray<TArray<int32>>& ParticleIndices);

		/*Debug Output*/
		CHAOS_API void DebugDrawSimulationData();
		CHAOS_API void DebugDrawTetrahedralParticles(FFleshThreadingProxy& Proxy);
		CHAOS_API void WriteFrame(FThreadingProxy& InProxy, const FSolverReal DeltaTime);
		CHAOS_API void WriteTrisGEO(const FSolverParticles& Particles, const TArray<TVec3<int32>>& Mesh);

		/*Game Thread API*/
		bool HasObject(UObject* InObject) const { return InitializedObjects_External.Contains(InObject); }
		CHAOS_API void AddProxy(FThreadingProxy* InObject);
		CHAOS_API void RemoveProxy(FThreadingProxy* InObject);
		CHAOS_API TUniquePtr<FDeformablePackage> PullOutputPackage();
		CHAOS_API void PushInputPackage(int32 Frame, FDeformableDataMap&& InPackage);
		CHAOS_API void PushRestartPackage(int32 InFrame, FDeformableDataMap&& InPackage);

		const FDeformableSolverProperties& GetProperties() const { return Property; }
		void ResetGaussSeidelVariables();

	private:

		// connections outside the solver.
		static CHAOS_API FCriticalSection	InitializationMutex; // @todo(flesh) : change to threaded commands to prevent the lock. 
		static CHAOS_API FCriticalSection	RemovalMutex; // @todo(flesh) : change to threaded commands to prevent the lock. 
		static CHAOS_API FCriticalSection	PackageOutputMutex;
		static CHAOS_API FCriticalSection	PackageInputMutex;
		static CHAOS_API FCriticalSection   PackageRestartMutex;
		static CHAOS_API FCriticalSection	SolverEnabledMutex;

		TArray< FThreadingProxy* > RemovedProxys_Internal;
		TArray< FThreadingProxy* > UninitializedProxys_Internal;
		TArray< TUniquePtr<FDeformablePackage>  > BufferedInputPackages;
		TArray< TUniquePtr<FDeformablePackage>  > BufferedOutputPackages;
		TArray< TUniquePtr<FDeformablePackage>  > BufferedRestartPackages;
		TUniquePtr < FDeformablePackage > CurrentInputPackage;
		TUniquePtr < FDeformablePackage > PreviousInputPackage;
		TUniquePtr < FDeformablePackage > CurrentRestartPackage;
		bool bPendingRestart = false;

		TSet< const UObject* > InitializedObjects_External;
		TMap< FThreadingProxy::FKey, TUniquePtr<FThreadingProxy> > Proxies;

		// User Configuration
		FDeformableSolverProperties Property;


		// Simulation Variables
		TUniquePtr<Softs::FPBDEvolution> Evolution;
		TArray<TUniquePtr<Softs::FXPBDCorotatedTetrahedralConstraints>> CorotatedConstraints;
		TUniquePtr<Softs::FGaussSeidelCorotatedTetrahedralConstraints> GSCorotatedConstraints;
		TUniquePtr<Softs::FGaussSeidelNeohookeanTetrahedralConstraints> GSNeohookeanConstraints;
		TUniquePtr<Softs::FGaussSeidelCorotatedTriangleConstraints> GSCorotatedCodConstraints;
		TUniquePtr<Softs::FGaussSeidelLinearTriangleConstraints> GSLinearCodConstraints;
		TUniquePtr<Softs::FGaussSeidelSpringConstraints> GSWeakConstraints;
		TUniquePtr<Softs::FGaussSeidelDynamicSpringConstraints> GSDynamicWeakConstraints;
		TUniquePtr<Softs::FGaussSeidelParticleSphereRepulsionConstraints> GSSphereRepulsionConstraints;
		TUniquePtr<Softs::FGaussSeidelUnilateralVolumeConstraints> GSVolumeConstraints;
		TArray<TUniquePtr<Softs::FXPBDWeightedSpringConstraints>> WeakConstraints;
		TArray<TUniquePtr<Softs::FBlendedXPBDCorotatedTetrahedralConstraints>> BlendedCorotatedConstraints;
		TUniquePtr<Softs::FXPBDGridBasedCorotatedTetrahedralConstraints> GridBasedCorotatedConstraint;
		TUniquePtr<Softs::FGaussSeidelMainConstraints> GSMainConstraint;
		TUniquePtr<Softs::FPBDCollisionSpringConstraints> CollisionSpringConstraint;
		TUniquePtr<Softs::FPBDTriangleMeshCollisions> TriangleMeshCollisions;
		TArrayCollectionArray<const UObject*> MObjects;
		TUniquePtr <TArray<TVec3<int32>>> SurfaceElements;
		TUniquePtr <TArray<TVec3<int32>>> TetmeshSurfaceElements;
		TUniquePtr <TArray<Chaos::TVec4<int32>>> AllElements;
		TUniquePtr <FTriangleMesh> SurfaceTriangleMesh;
		TUniquePtr <TArray<int32>> SurfaceCollisionVertices;
		TUniquePtr <TArray<TArray<int32>>> AllIncidentElements;
		TUniquePtr <TArray<TArray<int32>>> AllIncidentElementsLocal;
		TUniquePtr <TArray<FSolverReal>> AllTetEMeshArray;
		TUniquePtr <TArray<FSolverReal>> AllTetNuMeshArray;
		TUniquePtr <TArray<FSolverReal>> AllTetAlphaJArray;
		TUniquePtr <TArray<TArray<int32>>> AllIndices;
		TUniquePtr <TArray<TArray<int32>>> AllSecondIndices;
		TUniquePtr <TArray<FSolverReal>> AllWeights;
		TUniquePtr <TArray<FSolverReal>> AllSecondWeights;
		TUniquePtr <TArray<TVec3<int32>>> AllUnconstrainedSurfaceElementsCorotatedCod;  //correspond to the triangle mesh elements that are simulated using corotated cod
		TUniquePtr <TArray<TVec3<int32>>> AllUnconstrainedSurfaceElementsSkin;          //correspond to the triangle mesh elements that are simulated using linear cod constraints
		TUniquePtr <TArray<FSolverReal>> AllCorotatedCodEMeshArray;
		TUniquePtr <TArray<FSolverReal>> AllSkinEMeshArray;
		TUniquePtr <TArray<int32>> ParticleComponentIndex;
		TMap<int32, TSet<int32>> ParticleTriangleExclusionMap;
		//Muscle Activation Variables
		TUniquePtr<Softs::FMuscleActivationTetrahedralConstraints> MuscleActivationConstraints;
		TMap<FThreadingProxy::FKey, int32> MuscleIndexOffset;

		typedef TMap<int32, TTuple<float, float, FVector3f>> TransientConstraintBufferMap;
		TransientConstraintBufferMap TransientConstraintBuffer;

		bool bEnableSolver = true;
		FSolverReal Time = 0.f;
		int32 Frame = 0;
		int32 Iteration = 0;
		bool bSimulationInitialized = false;
		int32 GroupOffset = 1;
		TArray<TVector<int32, 2>, TInlineAllocator<8>> PrevEvolutionActiveRange;
		bool bDynamicConstraintIsUpdated = false;
	};


}; // namesapce Chaos::Softs
