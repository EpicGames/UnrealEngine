// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Chaos/Range.h"
#include "ChaosDeformableSolverProxy.generated.h"

/**
* Supported simulation spaces for the ChaosDeformable solver.
*/
UENUM()
enum ChaosDeformableSimSpace : uint8
{
	World		UMETA(DisplayName = "World"),
	ComponentXf	UMETA(DisplayName = "Component"), // Component fails on Mac
	Bone		UMETA(DisplayName = "Bone"),
};

namespace Chaos::Softs
{
	class FThreadingProxy {
		const UObject* Owner = nullptr;
		FName TypeName = FName();

	public:
		typedef const UObject* FKey;

		FThreadingProxy(const UObject* InOwner = nullptr, FName InTypeName = FName())
			: Owner(InOwner)
			, TypeName(InTypeName)
		{}
		virtual ~FThreadingProxy() = default;

		FName BaseTypeName() const { return TypeName; }

		template<class T>
		T* As() { return T::TypeName().IsEqual(BaseTypeName()) ? (T*)this : nullptr; }

		template<class T>
		T* As() const { return T::TypeName().IsEqual(BaseTypeName()) ? (T*)this : nullptr; }

		const UObject* GetOwner() const { return Owner; }


		class FBuffer
		{
			const UObject* Owner = nullptr;
			FName TypeName = FName();

		public:
			FBuffer(const FThreadingProxy& Ref)
				: Owner(Ref.GetOwner())
				, TypeName(Ref.BaseTypeName()) {}

			FBuffer(const UObject* InOwner,FName InTypeName)
				: Owner(InOwner)
				, TypeName(InTypeName) {}
			virtual ~FBuffer() = default;

			FName BaseTypeName() const { return TypeName; }

			template<class T>
			T* As() { return T::Source::TypeName().IsEqual(BaseTypeName()) ? (T*)this : nullptr; }

			template<class T>
			T* As() const { return T::Source::TypeName().IsEqual(BaseTypeName()) ? (T*)this : nullptr; }
		};
	};

	class FFleshThreadingProxy : public FThreadingProxy
	{
		typedef FThreadingProxy Super;

		// Component transform
		FTransform WorldToComponentXf = FTransform::Identity; // component xf
		FTransform ComponentToWorldXf = FTransform::Identity; // inv component xf

		// Bone transform
		FTransform ComponentToBoneXf = FTransform::Identity; // bone xf
		FTransform BoneToComponentXf = FTransform::Identity; // inv bone xf
		FTransform PrevComponentToBoneXf = FTransform::Identity; // bone xf
		FTransform PrevBoneToComponentXf = FTransform::Identity; // inv bone xf

		TSharedRef<const FManagedArrayCollection> Rest;
		FManagedArrayCollection Dynamic;
		Chaos::FRange SolverParticleRange = Chaos::FRange(0, 0);
		ChaosDeformableSimSpace SimSpace = ChaosDeformableSimSpace::World;
		bool bIsCached = false;

	public:

		FFleshThreadingProxy(
			UObject* InOwner,
			const FTransform& InWorldToComponentXf, // ComponentXf
			const FTransform& InComponentToBoneXf,
			const ChaosDeformableSimSpace InSimSpace,
			TSharedRef<const FManagedArrayCollection> InRest,
			const FManagedArrayCollection& InDynamic)
			: Super(InOwner, TypeName())
			, WorldToComponentXf(InWorldToComponentXf)
			, ComponentToBoneXf(InComponentToBoneXf)
			, Rest(InRest)
			, Dynamic(InDynamic)
			, SimSpace(InSimSpace)
		{
			ComponentToWorldXf = FTransform(WorldToComponentXf.ToMatrixWithScale().Inverse());
			if (SimSpace == ChaosDeformableSimSpace::Bone)
			{
				BoneToComponentXf = FTransform(ComponentToBoneXf.ToMatrixWithScale().Inverse());
			}
		}
		virtual ~FFleshThreadingProxy() = default;

		static FName TypeName() { return FName("Flesh"); }

		bool CanSimulate() const
		{
			return Rest->NumElements(FGeometryCollection::VerticesGroup) > 0;
		}

		bool IsBoneSpace() const
		{
			return SimSpace == ChaosDeformableSimSpace::Bone;
		}

		void SetIsCached(bool bInIsCached) { bIsCached = bInIsCached; }
		bool GetIsCached() const { return bIsCached; }

		//! Update the component and bone transforms with current data from the scene.
		void UpdateSimSpace(const FTransform& InWorldToComponentXf, const FTransform& InComponentToBoneXf)
		{
			WorldToComponentXf = InWorldToComponentXf;
			ComponentToWorldXf = FTransform(WorldToComponentXf.ToMatrixWithScale().Inverse());
			if (SimSpace == ChaosDeformableSimSpace::Bone)
			{
				PrevComponentToBoneXf = ComponentToBoneXf;
				PrevBoneToComponentXf = BoneToComponentXf;

				ComponentToBoneXf = InComponentToBoneXf;
				BoneToComponentXf = FTransform(ComponentToBoneXf.ToMatrixWithScale().Inverse());
			}
		}

		//! Return the transform required to get points from component space to whatever
		//! the specified simulation space is.
		const FTransform& GetInitialPointsTransform() const
		{
			switch (SimSpace)
			{
			case ChaosDeformableSimSpace::World:
				// Initial points are in component space, so apply the component xf.
				return WorldToComponentXf;
			case ChaosDeformableSimSpace::ComponentXf:
				// We're already in the right space.
				return FTransform::Identity;
			case ChaosDeformableSimSpace::Bone:
				// Points have ComponentToBoneXf baked in, so we invert it out to get
				// points in bone space.
				return BoneToComponentXf;
			default:
				return FTransform::Identity;
			}
		}

		const FTransform& GetCurrentPointsTransform() const
		{
			switch (SimSpace)
			{
			case ChaosDeformableSimSpace::World:
				return WorldToComponentXf;
			case ChaosDeformableSimSpace::ComponentXf:
				return FTransform::Identity;
			case ChaosDeformableSimSpace::Bone:
				return BoneToComponentXf;
			default:
				return FTransform::Identity;
			}
		}

		const FTransform& GetPreviousPointsTransform() const
		{
			switch (SimSpace)
			{
			case ChaosDeformableSimSpace::World:
				return WorldToComponentXf;
			case ChaosDeformableSimSpace::ComponentXf:
				return FTransform::Identity;
			case ChaosDeformableSimSpace::Bone:
				return PrevBoneToComponentXf;
			default:
				return FTransform::Identity;
			}
		}

		Chaos::TVector<Chaos::FRealSingle,3> RotateWorldSpaceVector(const Chaos::TVector<Chaos::FRealSingle, 3>& Dir) const
		{
			switch (SimSpace)
			{
			case ChaosDeformableSimSpace::World:
				return Dir;
			case ChaosDeformableSimSpace::ComponentXf:
			{
				return Chaos::TVector<Chaos::FRealSingle, 3>(
					ComponentToWorldXf.TransformVectorNoScale(FVector(Dir[0], Dir[1], Dir[2])));
			}
			case ChaosDeformableSimSpace::Bone:
			{
				FVector Tmp = ComponentToWorldXf.TransformVectorNoScale(FVector(Dir[0], Dir[1], Dir[2]));
				return Chaos::TVector<Chaos::FRealSingle, 3>(ComponentToBoneXf.TransformVectorNoScale(Tmp));
			}
			default:
				return Dir;
			}
		}

		//! Return the transform required to get from whatever our simulation space is, to
		//! component space.
		FTransform GetFinalTransform() const
		{
			switch (SimSpace)
			{
			case ChaosDeformableSimSpace::World:
				return ComponentToWorldXf;
			case ChaosDeformableSimSpace::ComponentXf:
				return FTransform::Identity;
			case ChaosDeformableSimSpace::Bone:
				return ComponentToBoneXf;
			default:
				return FTransform::Identity;
			}
		}

		void SetSolverParticleRange(int32 InStart, int32 InRange)
		{
			SolverParticleRange = Chaos::FRange(InStart, InRange);
		}

		const Chaos::FRange& GetSolverParticleRange() const
		{
			return SolverParticleRange;
		}

		FManagedArrayCollection& GetDynamicCollection()				{ return Dynamic; }
		const FManagedArrayCollection& GetDynamicCollection() const { return Dynamic; }
		const FManagedArrayCollection& GetRestCollection() const	{ return *Rest; }

		class FFleshInputBuffer : public FThreadingProxy::FBuffer
		{
			typedef FThreadingProxy::FBuffer Super;

		public:
			typedef FFleshThreadingProxy Source;

			FFleshInputBuffer(
				const FManagedArrayCollection&& InSimulationCollection,
				const FTransform& InWorldToComponentXf,
				const FTransform& InComponentToBoneXf,
				const int32 InSimSpaceBoneIndex,
				const bool InbEnableGravity,
				const float InStiffnessMultiplier, 
				const float InDampingMultiplier, 
				const float InMassMultiplier,
				const float InInCompressibilityMultiplier,
				const float InInflationMultiplier,
				const UObject* InOwner = nullptr)
				: Super(InOwner, FFleshThreadingProxy::TypeName())
				, SimulationCollection(InSimulationCollection)
				, WorldToComponentXf(InWorldToComponentXf)
				, ComponentToBoneXf(InComponentToBoneXf)
				, Transforms(TArray<FTransform>())
				, RestTransforms(TArray<FTransform>())
				, bEnableGravity(InbEnableGravity)
				, StiffnessMultiplier(InStiffnessMultiplier)
				, DampingMultiplier(InDampingMultiplier)
				, MassMultiplier(InMassMultiplier)
				, IncompressibilityMultiplier(InInCompressibilityMultiplier)
				, InflationMultiplier(InInflationMultiplier)
				, SimSpaceBoneIndex(InSimSpaceBoneIndex)
			{}

			FFleshInputBuffer(
				const FManagedArrayCollection&& InSimulationCollection,
				const FTransform& InWorldToComponentXf,
				const FTransform& InComponentToBoneXf, 
				const int32 InSimSpaceBoneIndex,
				const TArray<FTransform>&& InTransforms, 
				const TArray<FTransform>&& InRestTransforms, 
				const bool InbEnableGravity, 
				const float InStiffnessMultiplier, 
				const float InDampingMultiplier, 
				const float InMassMultiplier, 
				const float InInCompressibilityMultiplier,
				const float InInflationMultiplier,
				const UObject* InOwner = nullptr)
				: Super(InOwner, FFleshThreadingProxy::TypeName())
				, SimulationCollection(InSimulationCollection)
				, WorldToComponentXf(InWorldToComponentXf)
				, ComponentToBoneXf(InComponentToBoneXf)
				, Transforms(InTransforms)
				, RestTransforms(InRestTransforms)
				, bEnableGravity(InbEnableGravity)
				, StiffnessMultiplier(InStiffnessMultiplier)
				, DampingMultiplier(InDampingMultiplier)
				, MassMultiplier(InMassMultiplier)
				, IncompressibilityMultiplier(InInCompressibilityMultiplier)
				, InflationMultiplier(InInflationMultiplier)
				, SimSpaceBoneIndex(InSimSpaceBoneIndex)
			{}

			FFleshInputBuffer(
				const FManagedArrayCollection&& InSimulationCollection,
				const FTransform& InWorldToComponentXf,
				const FTransform& InComponentToBoneXf,
				const int32 InSimSpaceBoneIndex,
				const TArray<FTransform>&& InTransforms,
				const TArray<FTransform>&& InRestTransforms,
				const bool InbEnableGravity,
				const float InStiffnessMultiplier,
				const float InDampingMultiplier,
				const float InMassMultiplier,
				const float InInCompressibilityMultiplier,
				const float InInflationMultiplier,
				const TArray<int32>&& InMuscleIndices,
				const TArray<float>&& InMuscleActivation,
				const UObject* InOwner = nullptr)
				: Super(InOwner, FFleshThreadingProxy::TypeName())
				, SimulationCollection(InSimulationCollection)
				, WorldToComponentXf(InWorldToComponentXf)
				, ComponentToBoneXf(InComponentToBoneXf)
				, Transforms(InTransforms)
				, RestTransforms(InRestTransforms)
				, bEnableGravity(InbEnableGravity)
				, StiffnessMultiplier(InStiffnessMultiplier)
				, DampingMultiplier(InDampingMultiplier)
				, MassMultiplier(InMassMultiplier)
				, IncompressibilityMultiplier(InInCompressibilityMultiplier)
				, InflationMultiplier(InInflationMultiplier)
				, SimSpaceBoneIndex(InSimSpaceBoneIndex)
				, MuscleIndices(InMuscleIndices)
				, MuscleActivation(InMuscleActivation)
			{}

			virtual ~FFleshInputBuffer() = default;

			const FManagedArrayCollection SimulationCollection;

			FTransform WorldToComponentXf;
			FTransform ComponentToBoneXf;

			TArray<FTransform> Transforms;
			TArray<FTransform> RestTransforms; 
			bool bEnableGravity;
			float StiffnessMultiplier;
			float DampingMultiplier;
			float MassMultiplier;
			float IncompressibilityMultiplier;
			float InflationMultiplier;
			int32 SimSpaceBoneIndex = INDEX_NONE;
			TArray<int32> MuscleIndices;
			TArray<float> MuscleActivation;
		};

		class FFleshOutputBuffer : public FThreadingProxy::FBuffer
		{
			typedef FThreadingProxy::FBuffer Super;

		public:
			typedef FFleshThreadingProxy Source;
			FFleshOutputBuffer(const Source& Ref)
				: Super(Ref)
			{
				// Ref.Dynamic will have updated solver data 
				Dynamic.AddAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup);
				Dynamic.CopyMatchingAttributesFrom(Ref.GetDynamicCollection());
			}
			virtual ~FFleshOutputBuffer() = default;

			FManagedArrayCollection Dynamic;
		};

		class FFleshRestartBuffer : public FThreadingProxy::FBuffer
		{
			typedef FThreadingProxy::FBuffer Super;

		public:
			typedef FFleshThreadingProxy Source;
			FFleshRestartBuffer(const FManagedArrayCollection& InDynamicCollection,
				const UObject* InOwner = nullptr)
				: Super(InOwner, FFleshThreadingProxy::TypeName())
			{
				Dynamic.AddAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup);
				Dynamic.CopyMatchingAttributesFrom(InDynamicCollection);
			}
			virtual ~FFleshRestartBuffer() = default;

			FManagedArrayCollection Dynamic;
		};
	};
}// namespace Chaos::Softs
