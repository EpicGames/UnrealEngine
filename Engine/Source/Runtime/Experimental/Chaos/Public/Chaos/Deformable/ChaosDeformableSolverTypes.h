// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/DebugDrawQueue.h"
#include "Chaos/Deformable/ChaosDeformableSolverProxy.h"
#include "Chaos/Deformable/ChaosDeformableCollisionsProxy.h"
#include "Chaos/PBDSoftsEvolutionFwd.h"
#include "CoreMinimal.h"

class UDeformableSolverComponent;
class FFleshCacheAdapter;

namespace Chaos::Softs
{
	struct FDeformableSolverProperties
	{
		FDeformableSolverProperties() 
			: FDeformableSolverProperties(
				/* InNumSolverSubSteps */ 2,
				/* InNumSolverIterations */ 5,
				/* InFixTimeStep */ false,
				/* InTimeStepSize */ (FSolverReal)0.05,
				/* InCacheToFile */ false,
				/* InbEnableKinematics */ true,
				/* InbUseFloor */ true,
				/* InbUseGridBasedConstraints */ false,
				/* InGridDx */ (FSolverReal)1.0,
				/* InbDoQuasistatics */ false,
				/* InbDoBlended */ false,
				/* InBlendedZeta */ (FSolverReal)0.1,
				/* InbEnableGravity */ true,
				/* InbEnableCorotatedConstraints */ true,
				/* InbEnablePositionTargets */ true,
				/* InbUseGaussSeidelConstraints */ false,
				/* InbUseSOR */ true,
				/* InOmegaSOR */ (FSolverReal)1.6,
				/* InbUseGSNeohookean */ false,
				/* InbDoSpringCollision */ false,
				/* InbDoInComponentSpringCollision */ false,
				/* InNRingExcluded */ 1,
				/* InSpringCollisionSearchRadius */ (FSolverReal)0,
				/* InSpringCollisionStiffness */ (FSolverReal)500.0,
				/* InbAllowSliding */ true,
				/* InbDoSphereRepulsion */ false,
				/* InSphereRepulsionRadius */ (FSolverReal)0,
				/* InSphereRepulsionStiffness */ (FSolverReal)500.0,
				/* InbDoLengthBasedMuscleActivation */ false,
				/* InbOverrideMuscleActivationWithAnimatedCurves */ false,
				/* InbCollideWithFullMesh */ false,
				/* InbEnableDynamicSprings */ true) 
		{};

		FDeformableSolverProperties(
			int32 InNumSolverSubSteps,
			int32 InNumSolverIterations,
			bool InFixTimeStep,
			FSolverReal InTimeStepSize,
			bool InCacheToFile,
			bool InbEnableKinematics,
			bool InbUseFloor,
			bool InbUseGridBasedConstraints,
			FSolverReal InGridDx,
			bool InbDoQuasistatics,
			bool InbDoBlended,
			FSolverReal InBlendedZeta,
			bool InbEnableGravity,
			bool InbEnableCorotatedConstraints,
			bool InbEnablePositionTargets,
			bool InbUseGaussSeidelConstraints,
			bool InbUseSOR,
			FSolverReal InOmegaSOR,
			bool InbUseGSNeohookean,
			bool InbDoSpringCollision,
			bool InbDoInComponentSpringCollision,
			int32 InNRingExcluded,
			FSolverReal InSpringCollisionSearchRadius,
			FSolverReal InSpringCollisionStiffness,
			bool InbAllowSliding,
			bool InbDoSphereRepulsion,
			FSolverReal InSphereRepulsionRadius,
			FSolverReal InSphereRepulsionStiffness,
			bool InbDoLengthBasedMuscleActivation,
			bool InbOverrideMuscleActivationWithAnimatedCurves,
			bool InbCollideWithFullMesh,
			bool InbEnableDynamicSprings)
			: NumSolverSubSteps(InNumSolverSubSteps)
			, NumSolverIterations(InNumSolverIterations)
			, FixTimeStep(InFixTimeStep)
			, TimeStepSize(InTimeStepSize)
			, CacheToFile(InCacheToFile)
			, bEnableKinematics(InbEnableKinematics)
			, bUseFloor(InbUseFloor)
			, bUseGridBasedConstraints(InbUseGridBasedConstraints)
			, GridDx(InGridDx)
			, bDoQuasistatics(InbDoQuasistatics)
			, bDoBlended(InbDoBlended)
			, BlendedZeta(InBlendedZeta)
			, bEnableGravity(InbEnableGravity)
			, bEnableCorotatedConstraints(InbEnableCorotatedConstraints)
			, bEnablePositionTargets(InbEnablePositionTargets)
			, bUseGaussSeidelConstraints(InbUseGaussSeidelConstraints)
			, bUseSOR(InbUseSOR)
			, OmegaSOR(InOmegaSOR)
			, bUseGSNeohookean(InbUseGSNeohookean)
			, bDoSpringCollision(InbDoSpringCollision)
			, bDoInComponentSpringCollision(InbDoInComponentSpringCollision)
			, NRingExcluded(InNRingExcluded)
			, SpringCollisionSearchRadius(InSpringCollisionSearchRadius)
			, SpringCollisionStiffness(InSpringCollisionStiffness)
			, bAllowSliding(InbAllowSliding)
			, bDoSphereRepulsion(InbDoSphereRepulsion)
			, SphereRepulsionRadius(InSphereRepulsionRadius)
			, SphereRepulsionStiffness(InSphereRepulsionStiffness)
			, bDoLengthBasedMuscleActivation(InbDoLengthBasedMuscleActivation)
			, bOverrideMuscleActivationWithAnimatedCurves(InbOverrideMuscleActivationWithAnimatedCurves)
			, bCollideWithFullMesh(InbCollideWithFullMesh)
			, bEnableDynamicSprings(InbEnableDynamicSprings)
		{}

		int32 NumSolverSubSteps = 5;
		int32 NumSolverIterations = 5;
		bool FixTimeStep = false;
		FSolverReal TimeStepSize = (FSolverReal)0.05;
		bool CacheToFile = false;
		bool bEnableKinematics = true;
		bool bUseFloor = true;
		bool bUseGridBasedConstraints = false;
		FSolverReal GridDx = (FSolverReal)1.;
		bool bDoQuasistatics = false;
		bool bDoBlended = false;
		FSolverReal BlendedZeta = (FSolverReal)0.;
		bool bEnableGravity = true;
		bool bEnableCorotatedConstraints = true;
		bool bEnablePositionTargets = true;
		bool bUseGaussSeidelConstraints = false;
		bool bUseSOR = true;
		FSolverReal OmegaSOR = (FSolverReal)1.6;
		bool bUseGSNeohookean = false;
		bool bDoSpringCollision = false;
		bool bDoInComponentSpringCollision = false;
		int32 NRingExcluded = 1;
		FSolverReal SpringCollisionSearchRadius = (FSolverReal)0;
		FSolverReal SpringCollisionStiffness = (FSolverReal)500.;
		bool bAllowSliding = true;
		bool bDoSphereRepulsion = false;
		FSolverReal SphereRepulsionRadius = (FSolverReal)0;
		FSolverReal SphereRepulsionStiffness = (FSolverReal)500.;
		bool bDoLengthBasedMuscleActivation = false;
		bool bOverrideMuscleActivationWithAnimatedCurves = false;
		bool bCollideWithFullMesh = false;
		bool bEnableDynamicSprings = true; 
	};


	/*Data Transfer*/
	typedef TSharedPtr<const FThreadingProxy::FBuffer> FDataMapValue; // Buffer Pointer
	typedef TMap<FThreadingProxy::FKey, FDataMapValue > FDeformableDataMap; // <const UObject*,FBufferSharedPtr>

	struct FDeformablePackage {
		FDeformablePackage()
		{}

		FDeformablePackage(int32 InFrame, FDeformableDataMap&& InMap)
			: Frame(InFrame)
			, ObjectMap(InMap)
		{}

		int32 Frame = INDEX_NONE;
		FDeformableDataMap ObjectMap;
	};

	/* Accessor for the Game Thread*/
	class FGameThreadAccessor
	{
	public:
//		friend class UDeformableSolverComponent;
//		friend class FFleshCacheAdapter;
//#if PLATFORM_WINDOWS
//	protected:
//#endif
		FGameThreadAccessor() {}
	};


	/* Accessor for the Physics Thread*/
	class FPhysicsThreadAccessor
	{
	public:
//		friend class UDeformableSolverComponent;
//		friend class FFleshCacheAdapter;
//#if PLATFORM_WINDOWS
//	protected:
//#endif
		FPhysicsThreadAccessor() {}
	};


	struct FDeformableDebugParams
	{				
		bool bDoDrawTetrahedralParticles = false;
		bool bDoDrawKinematicParticles = false;
		bool bDoDrawTransientKinematicParticles = false;
		bool bDoDrawRigidCollisionGeometry = false;
		FSolverReal ParticleRadius = 5.f;

		bool IsDebugDrawingEnabled()
		{ 
#if WITH_EDITOR
			// p.Chaos.DebugDraw.Enabled 1
			return Chaos::FDebugDrawQueue::GetInstance().IsDebugDrawingEnabled();
#else
			return false;
#endif
		}
	};

	struct FDeformableXPBDCorotatedParams
	{
		int32 XPBDCorotatedBatchSize = 5;
		int32 XPBDCorotatedBatchThreshold = 5;
		int32 NumLogExtremeParticle = 0;
	};


}; // namesapce Chaos::Softs
