// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/PBDSpringConstraintsBase.h"
#include "ChaosStats.h"
#include "Chaos/ImplicitQRSVD.h"
#include "Chaos/GraphColoring.h"
#include "Chaos/Framework/Parallel.h"
#include "Chaos/Deformable/ChaosDeformableSolverTypes.h"
#include "Chaos/Vector.h"
#include "Chaos/SoftsSolverParticlesRange.h"


namespace Chaos::Softs
{
	// Triangle constraint using fixed corotated constitutive model [Stomakhin et al., 2012]
	// 3D triangles are projected to the 2D space for deformation gradient (3*2 matrix) computation
	class FGaussSeidelCorotatedTriangleConstraints
	{
	public:
		CHAOS_API FGaussSeidelCorotatedTriangleConstraints(
			const FSolverParticlesRange& InParticles,
			const TArray<TVector<int32, 3>>& InMesh,
			const bool bRecordMetricIn = true,
			const FSolverReal& EMesh = (FSolverReal)10.0,
			const FSolverReal& NuMesh = (FSolverReal).3);

		FGaussSeidelCorotatedTriangleConstraints(
			const FSolverParticles& InParticles,
			const TArray<TVector<int32, 3>>& InMesh,
			const bool bRecordMetricIn = true,
			const FSolverReal& EMesh = (FSolverReal)10.0,
			const FSolverReal& NuMesh = (FSolverReal).3)
			: FGaussSeidelCorotatedTriangleConstraints(
				FSolverParticlesRange(InParticles),
				InMesh, bRecordMetricIn, EMesh, NuMesh)
		{}

		CHAOS_API FGaussSeidelCorotatedTriangleConstraints(
			const FSolverParticlesRange& InParticles,
			const TArray<TVector<int32, 3>>& InMesh,
			const TArray<FSolverReal>& EMeshArray,
			const FSolverReal& NuMesh = (FSolverReal).3);

		FGaussSeidelCorotatedTriangleConstraints(
			const FSolverParticles& InParticles,
			const TArray<TVector<int32, 3>>& InMesh,
			const TArray<FSolverReal>& EMeshArray,
			const FSolverReal& NuMesh = (FSolverReal).3)
			: FGaussSeidelCorotatedTriangleConstraints(
				FSolverParticlesRange(InParticles),
				InMesh, EMeshArray, NuMesh)
		{}

		virtual ~FGaussSeidelCorotatedTriangleConstraints() = default;

		CHAOS_API TArray<TArray<int32>> GetConstraintsArray() const;

		CHAOS_API void AddHyperelasticResidualAndHessian(const FSolverParticlesRange& Particles, const int32 ElementIndex, const int32 ElementIndexLocal, const FSolverReal Dt, TVec3<FSolverReal>& ParticleResidual, Chaos::PMatrix<FSolverReal, 3, 3>& ParticleHessian);

	protected:

		PMatrix<FSolverReal, 3, 2> Ds(const int32 ElemIdx, const FSolverParticlesRange& InParticles) const;

		PMatrix<FSolverReal, 3, 2> F(const int32 ElemIdx, const FSolverParticlesRange& InParticles) const;

		CHAOS_API void ComputeNodalMass(const FSolverReal InDensity, const int32 NumParticles, TArray<FSolverReal>& NodalMass) const;

		void InitializeCodimensionData(const FSolverParticlesRange& Particles);

		static FSolverReal SafeRecip(const FSolverReal Len, const FSolverReal Fallback)
		{
			if (Len > (FSolverReal)UE_SMALL_NUMBER)
			{
				return (FSolverReal)1. / Len;
			}
			return Fallback;
		}

		static inline void SymSchur2D(const FSolverReal Aqq, const FSolverReal App, const FSolverReal Apq, FSolverReal& C, FSolverReal& S)
		{
			// A is an n x n matrix
			// (p, q) is an index pair satisfying 1 <= p < q <= n
			//
			// This function computes a (C, S) pair such that
			//
			// B = [ C, S]^T [a_pp, a_pq] [ C, S]
			//     [-S, C]   [a_pq, a_qq] [-S, C]
			//
			// is diagonal

			if (Apq != 0)
			{
				FSolverReal Tau = FSolverReal(0.5) * (Aqq - App) / Apq;
				FSolverReal T;
				if (Tau >= 0)
				{
					T = FSolverReal(1) / (Tau + FMath::Sqrt((FSolverReal)1. + Tau * Tau));
				}
				else
				{
					T = FSolverReal(-1) / (-Tau + FMath::Sqrt((FSolverReal)1. + Tau * Tau));
				}
				C = FSolverReal(1) / FMath::Sqrt((FSolverReal)1. + T * T);
				S = T * C;
			}
			else
			{
				C = (FSolverReal)1.;
				S = (FSolverReal)0.;
			}
		}

		static inline void Jacobi(const PMatrix<FSolverReal, 2, 2>& B, PMatrix<FSolverReal, 2, 2>& D, PMatrix<FSolverReal, 2, 2>& V)
		{
			FSolverReal C, S;
			SymSchur2D(B.M[3], B.M[0], B.M[1], C, S);

			V.M[0] = C;
			V.M[1] = -S;
			V.M[2] = S;
			V.M[3] = C;

			D = V.GetTransposed() * B * V;
		}

		CHAOS_API static PMatrix<FSolverReal, 3, 2> ComputeR(const PMatrix<FSolverReal, 3, 2>& Fe);

		CHAOS_API static PMatrix<FSolverReal, 3, 2> ComputeRSimple(const PMatrix<FSolverReal, 3, 2>& InputMatrix);

		//Does Gram Schmidt on a 3*2 matrix and returns an orthogonal one
		static Chaos::PMatrix<FSolverReal, 3, 2> GramSchmidt(const Chaos::PMatrix<FSolverReal, 3, 2>& InputMatrix)
		{
			TVec3<FSolverReal> Col1(InputMatrix.M[0], InputMatrix.M[1], InputMatrix.M[2]), Col2(InputMatrix.M[3], InputMatrix.M[4], InputMatrix.M[5]);
			const FSolverReal Col1Norm = Col1.Length();
			const FSolverReal OneOverCol1Norm = SafeRecip(Col1Norm, 0.f);
			TVec3<FSolverReal> Col1Normalized = Col1 * OneOverCol1Norm;
			TVec3<FSolverReal> Col2Orthogonal = Col2 - TVec3<FSolverReal>::DotProduct(Col1Normalized, Col2) * Col1Normalized;
			const FSolverReal Col2OrthogonalNorm = Col2Orthogonal.Length();
			const FSolverReal OneOverCol2OrthogonalNorm = SafeRecip(Col2OrthogonalNorm, 0.f);
			Col2Orthogonal *= OneOverCol2OrthogonalNorm;
			return Chaos::PMatrix<FSolverReal, 3, 2>(Col1Normalized[0], Col1Normalized[1], Col1Normalized[2], Col2Orthogonal[0], Col2Orthogonal[1], Col2Orthogonal[2]);
		}

		static PMatrix<FSolverReal, 2, 2> ComputeFTF(const PMatrix<FSolverReal, 3, 2>& InputMatrix)
		{
			return PMatrix<FSolverReal, 2, 2>(InputMatrix.M[0] * InputMatrix.M[0] + InputMatrix.M[1] * InputMatrix.M[1] + InputMatrix.M[2] * InputMatrix.M[2],
				InputMatrix.M[0] * InputMatrix.M[3] + InputMatrix.M[1] * InputMatrix.M[4] + InputMatrix.M[2] * InputMatrix.M[5],
				InputMatrix.M[3] * InputMatrix.M[3] + InputMatrix.M[4] * InputMatrix.M[4] + InputMatrix.M[5] * InputMatrix.M[5]);
		}

		//Computes det(FTF)^{1/2} * F * (FTF)^-1
		static void dJdF32(const PMatrix<FSolverReal, 3, 2>& F, PMatrix<FSolverReal, 3, 2>& dJ)
		{
			const PMatrix<FSolverReal, 2, 2> FTF = ComputeFTF(F);
			const FSolverReal J2 = FTF.Determinant();

			const PMatrix<FSolverReal, 2, 2> J2invT(FTF.M[3], -FTF.M[2], -FTF.M[1], FTF.M[0]);

			if (J2 > FSolverReal(0))
			{
				dJ = (FSolverReal(1) / FMath::Sqrt(J2)) *  (F * J2invT);
			}
			else
			{
				dJ = PMatrix<FSolverReal, 3, 2>(TVec3<FSolverReal>((FSolverReal)0.), TVec3<FSolverReal>((FSolverReal)0.));
			}
		}

	protected:
		mutable TArray<FSolverMatrix22> DmInverse;

		//material constants calculated from E:
		FSolverReal Mu;
		FSolverReal Lambda;
		TArray<FSolverReal> MuElementArray;
		TArray<FSolverReal> LambdaElementArray;
		TArray<FSolverReal> AlphaJArray;
		bool bRecordMetric;

		TArray<TVector<int32, 3>> MeshConstraints;
		mutable TArray<FSolverReal> Measure;
	};

	template <typename T, typename ParticleType>
	using FGaussSeidelCorotatedCodimensionalConstraints UE_DEPRECATED(5.8, "Deprecated. this class is to be deleted, use FGaussSeidelCorotatedTriangleConstraints instead") = FGaussSeidelCorotatedTriangleConstraints;

}  // End namespace Chaos::Softs

