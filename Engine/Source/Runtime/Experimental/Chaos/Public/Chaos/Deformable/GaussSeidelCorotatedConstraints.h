// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/PBDSpringConstraintsBase.h"
#include "Chaos/XPBDCorotatedConstraints.h"
#include "ChaosStats.h"
#include "Chaos/ImplicitQRSVD.h"
#include "Chaos/GraphColoring.h"
#include "Chaos/NewtonCorotatedCache.h"
#include "Chaos/Framework/Parallel.h"
#include "Chaos/Deformable/GaussSeidelWeakConstraints.h"


namespace Chaos::Softs
{
	// Tetrahedral constraint using fixed corotated constitutive model [Stomakhin et al., 2012]
	// for position-based nonlinear Gauss Seidel solver [Chen et al., 2024]
	class FGaussSeidelCorotatedTetrahedralConstraints : public FXPBDCorotatedTetrahedralConstraints
	{
		typedef FXPBDCorotatedTetrahedralConstraints Base;
		using Base::MeshConstraints;
		using Base::LambdaArray;
		using Base::Measure;
		using Base::DmInverse;
		using Base::MuElementArray;
		using Base::LambdaElementArray;
		using Base::CorotatedParams;
		using Base::F;

	public:
		CHAOS_API FGaussSeidelCorotatedTetrahedralConstraints(
			const FSolverParticlesRange& InParticles,
			const TArray<TVector<int32, 4>>& InMesh,
			const TArray<FSolverReal>& EMeshArray,
			const TArray<FSolverReal>& NuMeshArray,
			TArray<FSolverReal>&& AlphaJMeshArray,
			TArray<TArray<int32>>&& IncidentElementsIn,
			TArray<TArray<int32>>&& IncidentElementsLocalIn,
			const int32 ParticleStartIndexIn,
			const int32 ParticleEndIndexIn,
			const bool bDoQuasistaticsIn = false,
			const bool bDoSORIn = true,
			const FSolverReal InOmegaSOR = (FSolverReal)1.6,
			const FDeformableXPBDCorotatedParams& InParams = FDeformableXPBDCorotatedParams(),
			const FSolverReal& NuMesh = (FSolverReal).3,
			const bool bRecordMetricIn = false);

		CHAOS_API FGaussSeidelCorotatedTetrahedralConstraints(
			const FSolverParticles& InParticles,
			const TArray<TVector<int32, 4>>& InMesh,
			const TArray<FSolverReal>& EMeshArray,
			const TArray<FSolverReal>& NuMeshArray,
			TArray<FSolverReal>&& AlphaJMeshArray,
			TArray<TArray<int32>>&& IncidentElementsIn,
			TArray<TArray<int32>>&& IncidentElementsLocalIn,
			const int32 ParticleStartIndexIn,
			const int32 ParticleEndIndexIn,
			const bool bDoQuasistaticsIn = false,
			const bool bDoSORIn = true,
			const FSolverReal InOmegaSOR = (FSolverReal)1.6,
			const FDeformableXPBDCorotatedParams& InParams = FDeformableXPBDCorotatedParams(),
			const FSolverReal& NuMesh = (FSolverReal).3,
			const bool bRecordMetricIn = false);

		virtual ~FGaussSeidelCorotatedTetrahedralConstraints() = default;

		CHAOS_API void Apply(FSolverParticlesRange& Particles, const FSolverReal Dt) const;

		CHAOS_API void Apply(FSolverParticles& Particles, const FSolverReal Dt) const;

		CHAOS_API void Init(const FSolverReal Dt, const FSolverParticlesRange& Particles) const;

		CHAOS_API void Init(const FSolverReal Dt, const FSolverParticles& Particles) const;

		void Init() const override {}

		TArray<TArray<int32>>& GetIncidentElements() { return IncidentElements; }
		TArray<TArray<int32>>& GetIncidentElementsLocal() { return IncidentElementsLocal; }
		TArray<TVector<int32, 4>> GetMeshConstraints() const { return MeshConstraints; }
		void SetParticlesPerColor(TArray<TArray<int32>>&& InParticlesPerColor) { ParticlesPerColor = MoveTemp(InParticlesPerColor); }

		CHAOS_API TArray<TArray<int32>> GetMeshArray() const;

		CHAOS_API void AddHyperelasticResidualAndHessian(const FSolverParticlesRange& Particles, const int32 ElementIndex, const int32 ElementIndexLocal, const FSolverReal Dt, TVec3<FSolverReal>& ParticleResidual, Chaos::PMatrix<FSolverReal, 3, 3>& ParticleHessian);

	protected:
		CHAOS_API void InitColor(const FSolverParticlesRange& Particles);
		CHAOS_API void InitColor(const FSolverParticles& Particles);
		CHAOS_API void InitializeCorotatedLambdas();

		CHAOS_API void ApplySOR(FSolverParticlesRange& Particles, const FSolverReal Dt) const;

		CHAOS_API Chaos::TVector<FSolverReal, 3> ComputePerParticleResidual(const int32 ParticleIdx, const int32 IncidentIndex, const FSolverParticlesRange& Particles,
			const FSolverReal Dt, const bool AddMass = true) const;

		CHAOS_API Chaos::PMatrix<FSolverReal, 3, 3> ComputePerParticleCorotatedHessianSimple(const int32 ParticleIdx, const int32 IncidentIndex, const FSolverParticlesRange& Particles,
			const FSolverReal Dt, const bool AddMass = true) const;

	private:
		TVector<FSolverReal, 3> ComputeDeltax(const int32 ParticleIdx, const int32 IncidentIndex, const FSolverParticlesRange& Particles,
			const FSolverReal Dt) const;

	protected:
		TArray<TArray<int32>> IncidentElements;
		TArray<TArray<int32>> IncidentElementsLocal;
		FSolverReal LocalNewtonTol = FSolverReal(1e-5);
		TArray<TArray<int32>> ParticlesPerColor;
		int32 ParticleStartIndex;
		int32 ParticleEndIndex;
		mutable TArray<Chaos::TVector<FSolverReal, 3>> xtilde;
		mutable TArray<Chaos::TVector<FSolverReal, 3>> X_k_1;
		mutable TArray<Chaos::TVector<FSolverReal, 3>> X_k;
		bool bDoQuasistatics = false;
		bool bDoSOR = true;
		FSolverReal OmegaSOR = FSolverReal(1.6);
		mutable int32 CurrentIt = 0;
		TFunction<void(const Chaos::PMatrix<FSolverReal, 3, 3>&, const FSolverReal, const FSolverReal, Chaos::PMatrix<FSolverReal, 3, 3>&)> ComputeStress;
		TFunction<void(const Chaos::PMatrix<FSolverReal, 3, 3>&, const Chaos::PMatrix<FSolverReal, 3, 3>&, const FSolverReal, const FSolverReal, const int32, const FSolverReal, Chaos::PMatrix<FSolverReal, 3, 3>&)> ComputeHessianHelper;

	public:
		TFunction<void(const FSolverParticlesRange&, const int32, const FSolverReal, TVec3<FSolverReal>&)> AddAdditionalRes;
		TFunction<void(const FSolverParticlesRange&, const int32, const FSolverReal, Chaos::PMatrix<FSolverReal, 3, 3>&)> AddAdditionalHessian;
		TUniquePtr<TArray<int32>> ParticleColors;
	};

	template <typename T, typename ParticleType>
	using FGaussSeidelCorotatedConstraints UE_DEPRECATED(5.8, "Deprecated. this class is to be deleted, use FGaussSeidelCorotatedTetrahedralConstraints instead") = FGaussSeidelCorotatedTetrahedralConstraints;

}// End namespace Chaos::Softs
