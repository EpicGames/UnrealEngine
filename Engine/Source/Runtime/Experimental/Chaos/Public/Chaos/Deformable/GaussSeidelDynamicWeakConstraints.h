// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/BoundingVolumeHierarchy.h"
#include "Chaos/PBDSpringConstraintsBase.h"
#include "Chaos/XPBDCorotatedConstraints.h"
#include "ChaosStats.h"
#include "Chaos/ImplicitQRSVD.h"
#include "Chaos/GraphColoring.h"
#include "Chaos/NewtonCorotatedCache.h"
#include "Chaos/Framework/Parallel.h"
#include "Chaos/Utilities.h"
#include "Chaos/DebugDrawQueue.h"
#include "Chaos/XPBDWeakConstraints.h"
#include "Chaos/Triangle.h"
#include "Chaos/TriangleCollisionPoint.h"
#include "Chaos/TriangleMesh.h"
#include "Chaos/Deformable/GaussSeidelWeakConstraints.h"
#include "GeometryCollection/Facades/CollectionTetrahedralFacade.h"
namespace Chaos::Softs
{
	using Chaos::TVec3;
	struct FGaussSeidelDynamicSpringConstraints: public FGaussSeidelSpringConstraints
	{

		typedef FGaussSeidelSpringConstraints Base;
		using Base::ConstraintsData;
		using Base::NodalWeights;
		using Base::WCIncidentElements;
		using Base::WCIncidentElementsLocal;
		typedef typename FGaussSeidelSpringConstraints::FGaussSeidelConstraintHandle GSConstraintHandle;

		FGaussSeidelDynamicSpringConstraints(const FDeformableXPBDWeightedSpringConstraintParams& InParams)
			: Base({}, {}, {}, {}, {}, {}, {}, InParams)
		{
			ConstraintsData.AddArray(&Handles);
		}

		virtual ~FGaussSeidelDynamicSpringConstraints()
		{
			for (GSConstraintHandle* Handle:Handles)
			{
				delete Handle;
			}
		}
		
		const GSConstraintHandle* AddSingleParticleTetrahedraConstraint(const GeometryCollection::Facades::FTetrahedralFacade& InTargetGeom, const FSolverParticles& AllParticles, const GeometryCollection::Facades::TetrahedralParticleEmbedding& InIntersection, const int32 InConstraintIndex, const Chaos::FRange& SourceRange, const Chaos::FRange& TargetRange, const FSolverReal PositionTargetStiffness)
		{
			GSConstraintHandle* NewConstraintPtr = new GSConstraintHandle();
			NewConstraintPtr -> ConstraintIndex = InConstraintIndex;
			Handles[InConstraintIndex] = NewConstraintPtr;

			const FIntVector4 TargetTet = InTargetGeom.Tetrahedron[InTargetGeom.TetrahedronStart[InIntersection.GeometryIndex] + InIntersection.TetrahedronIndex] + FIntVector4(TargetRange.Start);

			FGaussSeidelSpringConstraintSingleData SingleConstraintData;
			SingleConstraintData.SingleIndices = { InIntersection.ParticleIndex + SourceRange.Start };
			SingleConstraintData.SingleSecondIndices =  {TargetTet[0], TargetTet[1], TargetTet[2], TargetTet[3]};
			SingleConstraintData.SingleWeights = {(FSolverReal)1.};
			SingleConstraintData.SingleSecondWeights = {InIntersection.BarycentricWeights[0], InIntersection.BarycentricWeights[1], InIntersection.BarycentricWeights[2], InIntersection.BarycentricWeights[3]};
			SingleConstraintData.bIsAnisotropic = false;
			SingleConstraintData.SingleNormal = TVec3<FSolverReal>((FSolverReal)0.);
			float ConstraintStiffness = 0.f;
			for (int32 k = 0; k < 4; k++)
			{
				ConstraintStiffness += SingleConstraintData.SingleSecondWeights[k] * PositionTargetStiffness * AllParticles.M(TargetTet[k]);
			}
			ConstraintStiffness += PositionTargetStiffness * AllParticles.M(SingleConstraintData.SingleIndices[0]);
			SingleConstraintData.SingleStiffness = (FSolverReal)ConstraintStiffness;

			ConstraintsData.SetSingleConstraint(SingleConstraintData, InConstraintIndex);

			return NewConstraintPtr;
			
		}

		void RemoveSingleConstraint(const GSConstraintHandle* DeletedSingleConstraint)
		{
			const int32 OldConstraintIndex = DeletedSingleConstraint->ConstraintIndex;

			const FGaussSeidelSpringConstraintSingleData& SingleData = ConstraintsData.GetSingleConstraintData(OldConstraintIndex);

			delete DeletedSingleConstraint;

			ConstraintsData.RemoveConstraint(OldConstraintIndex);
			if (ConstraintsData.Size() > 0)
			{
				Handles[OldConstraintIndex]->ConstraintIndex = OldConstraintIndex;
			}
			//Update incident element information:
			const int32 IndicesOffset = SingleData.SingleIndices.Num();
			for (int32 i = 0; i < SingleData.SingleIndices.Num(); i++)
			{
				const int32 ParticleIndex = SingleData.SingleIndices[i];
				for (int32 j = 0; j < WCIncidentElements[ParticleIndex].Num(); j++)
				{
					if (WCIncidentElements[ParticleIndex][j] == i && WCIncidentElementsLocal[ParticleIndex][j] == i)
					{
						WCIncidentElements[ParticleIndex].Remove(j);
						WCIncidentElementsLocal[ParticleIndex].Remove(j);
						break;
					}
				}
			}
			for (int32 i = 0; i < SingleData.SingleSecondIndices.Num(); i++)
			{
				const int32 ParticleIndex = SingleData.SingleSecondIndices[i];
				for (int32 j = 0; j < WCIncidentElements[ParticleIndex].Num(); j++)
				{
					if (WCIncidentElements[ParticleIndex][j] == i && WCIncidentElementsLocal[ParticleIndex][j] == i + IndicesOffset)
					{
						WCIncidentElements[ParticleIndex].Remove(j);
						WCIncidentElementsLocal[ParticleIndex].Remove(j);
						break;
					}
				}
			}
			
		}

		TArray<const GSConstraintHandle*> AddParticleTetrahedraConstraints(const GeometryCollection::Facades::FTetrahedralFacade& InTargetGeom, const FSolverParticles& AllParticles, const TArray<GeometryCollection::Facades::TetrahedralParticleEmbedding>& InIntersections, const Chaos::FRange& SourceRange, const Chaos::FRange& TargetRange, const FSolverReal ConstraintStiffness)
		{
			TArray<const GSConstraintHandle*> ConstraintHandles;
			ConstraintHandles.SetNum(InIntersections.Num());
		

			const int32 NumConstraintsOffset = ConstraintsData.Size();
			ConstraintsData.AddConstraints(InIntersections.Num());
			for (int32 i = 0; i < InIntersections.Num(); i++)
			{
				ConstraintHandles[i] = AddSingleParticleTetrahedraConstraint(InTargetGeom, AllParticles, InIntersections[i], NumConstraintsOffset + i, SourceRange, TargetRange, ConstraintStiffness);
			}
			Base::ComputeInitialWCData(AllParticles);
			return ConstraintHandles;
		}

		void RemoveConstraints(const TArray<const GSConstraintHandle*>& DeletedPtrs)
		{
			for (const GSConstraintHandle* DeletedPtr: DeletedPtrs )
			{
				RemoveSingleConstraint(DeletedPtr);
			}
		}

		void ReComputeNodalWeights(const TSet<int32>& DirtyVerts)
		{
			for (const int32 Vert : DirtyVerts)
			{
				check(Vert < NodalWeights.Num());
				NodalWeights[Vert].Init(FSolverReal(0), 6);
				for (int32 j = 0; j < WCIncidentElements[Vert].Num(); j++)
				{
					const int32 ConstraintIndex = WCIncidentElements[Vert][j];
					const int32 LocalIndex = WCIncidentElementsLocal[Vert][j];

					FSolverReal Weight = FSolverReal(0);
					if (LocalIndex >= ConstraintsData.GetIndices(ConstraintIndex).Num())
					{
						Weight = ConstraintsData.GetSecondWeights(ConstraintIndex)[LocalIndex - ConstraintsData.GetIndices(ConstraintIndex).Num()];
					}
					else
					{
						Weight = ConstraintsData.GetWeights(ConstraintIndex)[LocalIndex];
					}

					if (ConstraintsData.GetIsAnisotropic(ConstraintIndex))
					{
						for (int32 alpha = 0; alpha < 3; alpha++) 
						{
							NodalWeights[Vert][alpha] += ConstraintsData.GetNormal(ConstraintIndex)[alpha] * ConstraintsData.GetNormal(ConstraintIndex)[alpha] * Weight * Weight * ConstraintsData.GetStiffness(ConstraintIndex);
						}

						NodalWeights[Vert][3] += ConstraintsData.GetNormal(ConstraintIndex)[0] * ConstraintsData.GetNormal(ConstraintIndex)[1] * Weight * Weight * ConstraintsData.GetStiffness(ConstraintIndex);
						NodalWeights[Vert][4] += ConstraintsData.GetNormal(ConstraintIndex)[0] * ConstraintsData.GetNormal(ConstraintIndex)[2] * Weight * Weight * ConstraintsData.GetStiffness(ConstraintIndex);
						NodalWeights[Vert][5] += ConstraintsData.GetNormal(ConstraintIndex)[1] * ConstraintsData.GetNormal(ConstraintIndex)[2] * Weight * Weight * ConstraintsData.GetStiffness(ConstraintIndex);
					}
					else
					{
						for (int32 alpha = 0; alpha < 3; alpha++)
						{
							NodalWeights[Vert][alpha] += Weight * Weight * ConstraintsData.GetStiffness(ConstraintIndex);
						}
					}
				}
			}
		}

		void AdjustStiffness(const TArray<const GSConstraintHandle*> ConstraintHandles, const TArray<FSolverReal>& StiffnessPerConstraint, const FSolverParticles& Particles)
		{
			TSet<int32> DirtyVertices;
			for (int32 i = 0; i < ConstraintHandles.Num(); i++)
			{
				const GSConstraintHandle* SingleHandle = ConstraintHandles[i];
				const FSolverReal NewStiffness = StiffnessPerConstraint[i];
				const int32 ConstraintIndex = SingleHandle->ConstraintIndex;
				const FGaussSeidelSpringConstraintSingleData& SingleConstraintData = ConstraintsData.GetSingleConstraintData(ConstraintIndex);
				FSolverReal NewConstraintStiffness = (FSolverReal)0.;
				for (int32 k = 0; k < SingleConstraintData.SingleWeights.Num(); k++)
				{
					NewConstraintStiffness += SingleConstraintData.SingleWeights[k] * NewStiffness * Particles.M(SingleConstraintData.SingleIndices[k]);
				}
				for (int32 k = 0; k < SingleConstraintData.SingleSecondWeights.Num(); k++)
				{
					NewConstraintStiffness += SingleConstraintData.SingleSecondWeights[k] * NewStiffness * Particles.M(SingleConstraintData.SingleSecondIndices[k]);
				}
				ConstraintsData.SetStiffness(ConstraintIndex, NewConstraintStiffness);
				DirtyVertices.Append(SingleConstraintData.SingleIndices);
				DirtyVertices.Append(SingleConstraintData.SingleSecondIndices);
			}
			ReComputeNodalWeights(DirtyVertices);
		}

		TArrayCollectionArray<GSConstraintHandle*> Handles; //same size as number of constraints 

	};

	template <typename T, typename ParticleType>
	using FGaussSeidelDynamicWeakConstraints UE_DEPRECATED(5.8, "Deprecated. this struct is to be deleted, use FGaussSeidelDynamicSpringConstraints instead") = FGaussSeidelDynamicSpringConstraints;

} //End namespace Chaos::Softs
