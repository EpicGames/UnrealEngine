// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/PBDSpringConstraintsBase.h"
#include "ChaosStats.h"
#include "Chaos/ImplicitQRSVD.h"
#include "Chaos/GraphColoring.h"
#include "Chaos/Framework/Parallel.h"
#include "Chaos/Deformable/ChaosDeformableSolverTypes.h"
#include "Chaos/Vector.h"
#include "Chaos/SoftsSolverParticlesRange.h"


namespace Chaos::Softs
{
	// Triangle constraint using linear constitutive model
	// 3D triangles are projected to the 2D space for deformation gradient (3*2 matrix) computation
	class FGaussSeidelLinearTriangleConstraints
	{

	public:
		CHAOS_API FGaussSeidelLinearTriangleConstraints(
			const FSolverParticles& InParticles,
			const TArray<TVector<int32, 3>>& InMesh,
			const FSolverReal& EMesh = (FSolverReal)10.0,
			const FSolverReal& NuMesh = (FSolverReal).3);

		CHAOS_API FGaussSeidelLinearTriangleConstraints(
			const FSolverParticles& InParticles,
			const TArray<TVector<int32, 3>>& InMesh,
			const TArray<FSolverReal>& EMeshArray,
			const FSolverReal& NuMesh = (FSolverReal).3);

		virtual ~FGaussSeidelLinearTriangleConstraints() = default;

		CHAOS_API TArray<TArray<int32>> GetConstraintsArray() const;

		CHAOS_API void AddHyperelasticResidualAndHessian(const FSolverParticlesRange& Particles, const int32 ElementIndex, const int32 ElementIndexLocal, const FSolverReal Dt, TVec3<FSolverReal>& ParticleResidual, Chaos::PMatrix<FSolverReal, 3, 3>& ParticleHessian);

	protected:

		PMatrix<FSolverReal, 3, 2> Ds(const int32 ElemIdx, const FSolverParticlesRange& InParticles) const;

		PMatrix<FSolverReal, 3, 2> F(const int32 ElemIdx, const FSolverParticlesRange& InParticles) const;

		void InitializeCodimensionData(const FSolverParticlesRange& Particles);

		static FSolverReal SafeRecip(const FSolverReal Len, const FSolverReal Fallback)
		{
			if (Len > (FSolverReal)UE_SMALL_NUMBER)
			{
				return (FSolverReal)1. / Len;
			}
			return Fallback;
		}

	protected:
		mutable TArray<FSolverMatrix22> DmInverse;

		//material constants calculated from E:
		FSolverReal Mu;
		FSolverReal Lambda;
		TArray<FSolverReal> MuElementArray;
		TArray<FSolverReal> LambdaElementArray;
		TArray<FSolverReal> AlphaJArray;

		TArray<TVector<int32, 3>> MeshConstraints;
		mutable TArray<FSolverReal> Measure;
	};

	template <typename T, typename ParticleType>
	using FGaussSeidelLinearCodimensionalConstraints UE_DEPRECATED(5.8, "Deprecated. this class is to be deleted, use FGaussSeidelLinearTriangleConstraints instead") = FGaussSeidelLinearTriangleConstraints;

}  // End namespace Chaos::Softs


