// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/PBDSpringConstraintsBase.h"
#include "Chaos/XPBDCorotatedConstraints.h"
#include "ChaosStats.h"
#include "Chaos/ImplicitQRSVD.h"
#include "Chaos/GraphColoring.h"
#include "Chaos/NewtonCorotatedCache.h"
#include "Chaos/Framework/Parallel.h"
#include "Chaos/Deformable/GaussSeidelWeakConstraints.h"
#include "Chaos/Deformable/GaussSeidelCorotatedConstraints.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "Chaos/SoftsSolverParticlesRange.h"


#define PERF_SCOPE(X) SCOPE_CYCLE_COUNTER(X); TRACE_CPUPROFILER_EVENT_SCOPE(X);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.GSMainConstraint.Apply"), STAT_ChaosGSMainConstraint_Apply, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.GSMainConstraint.Acceleration"), STAT_ChaosGSMainConstraint_Acceleration, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.GSMainConstraint.Init"), STAT_ChaosGSMainConstraint_Init, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.GSMainConstraint.InitTransientColor"), STAT_ChaosGSMainConstraint_InitTransientColor, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos.Deformable.GSMainConstraint.InitDynamicColor"), STAT_ChaosGSMainConstraint_InitDynamicColor, STATGROUP_Chaos);

CHAOS_API DECLARE_LOG_CATEGORY_EXTERN(LogDeformableGaussSeidelMainConstraint, Log, All);

namespace Chaos::Softs
{
	// TFunction type aliases for Gauss-Seidel constraint lambdas
	// (Particles, ParticleIdx, Dt, OutResidual, OutHessian)
	using FGSInitialResidualHessianFunc = TFunction<void(const FSolverParticlesRange&, const int32, const FSolverReal, TVec3<FSolverReal>&, Chaos::PMatrix<FSolverReal, 3, 3>&)>;
	// (Particles, ConstraintIndex, LocalIndex, Dt, OutResidual, OutHessian)
	using FGSConstraintResidualHessianFunc = TFunction<void(const FSolverParticlesRange&, const int32, const int32, const FSolverReal, TVec3<FSolverReal>&, Chaos::PMatrix<FSolverReal, 3, 3>&)>;
	// (SourceParticles, TargetParticles, ConstraintIndex, LocalIndex, Dt, OutResidual, OutHessian)
	using FGSCrossBodyResidualHessianFunc = TFunction<void(const FSolverParticlesRange&, const FSolverParticlesRange&, const int32, const int32, const FSolverReal, TVec3<FSolverReal>&, Chaos::PMatrix<FSolverReal, 3, 3>&)>;
	// (ParticleIdx, Dt, OutHessian)
	using FGSPerNodeHessianFunc = TFunction<void(const int32, const FSolverReal, Chaos::PMatrix<FSolverReal, 3, 3>&)>;
	// (Particles, Dx, OutForceDifferential)
	using FGSForceDifferentialFunc = TFunction<void(const FSolverParticlesRange&, const TArray<TVec3<FSolverReal>>&, TArray<TVec3<FSolverReal>>&)>;

	class FGaussSeidelMainConstraints
	{

	public:
		CHAOS_API FGaussSeidelMainConstraints(
			const FSolverParticlesRange& InParticles,
			const bool bDoQuasistaticsIn = false,
			const bool bDoSORIn = true,
			const FSolverReal InOmegaSOR = (FSolverReal)1.6,
			const int32 ParallelMaxIn = 1000,
			const FSolverReal MaxDxRatioIn = FSolverReal(1),
			const FDeformableXPBDCorotatedParams& InParams = FDeformableXPBDCorotatedParams());

		CHAOS_API FGaussSeidelMainConstraints(
			const FSolverParticles& InParticles,
			const bool bDoQuasistaticsIn = false,
			const bool bDoSORIn = true,
			const FSolverReal InOmegaSOR = (FSolverReal)1.6,
			const int32 ParallelMaxIn = 1000,
			const FSolverReal MaxDxRatioIn = FSolverReal(1),
			const FDeformableXPBDCorotatedParams& InParams = FDeformableXPBDCorotatedParams());

		CHAOS_API FGaussSeidelMainConstraints(const FSolverParticlesRange& InParticles, const FCollectionPropertyConstFacade& Property, const FSolverReal MaxDxRatioIn);

		CHAOS_API FGaussSeidelMainConstraints(const FSolverParticles& InParticles, const FCollectionPropertyConstFacade& Property, const FSolverReal MaxDxRatioIn);

		virtual ~FGaussSeidelMainConstraints() = default;

		CHAOS_API void Resize(const int32 NewSize);

		const TArray<FGSConstraintResidualHessianFunc>& StaticConstraintResidualAndHessian() const { return AddStaticConstraintResidualAndHessian; }
		TArray<FGSConstraintResidualHessianFunc>& StaticConstraintResidualAndHessian() { return AddStaticConstraintResidualAndHessian; }
		const TArray<FGSConstraintResidualHessianFunc>& TransientConstraintResidualAndHessian() const { return AddTransientConstraintResidualAndHessian; }
		TArray<FGSConstraintResidualHessianFunc>& TransientConstraintResidualAndHessian() { return AddTransientConstraintResidualAndHessian; }
		const TArray<FGSConstraintResidualHessianFunc>& DynamicConstraintResidualAndHessian() const { return AddDynamicConstraintResidualAndHessian; }
		TArray<FGSConstraintResidualHessianFunc>& DynamicConstraintResidualAndHessian() { return AddDynamicConstraintResidualAndHessian; }

		const TArray<FGSCrossBodyResidualHessianFunc>& CrossBodyStaticConstraintResidualAndHessian() const { return AddCrossBodyStaticConstraintResidualAndHessian; }
		TArray<FGSCrossBodyResidualHessianFunc>& CrossBodyStaticConstraintResidualAndHessian() { return AddCrossBodyStaticConstraintResidualAndHessian; }
		const TArray<FGSCrossBodyResidualHessianFunc>& CrossBodyTransientConstraintResidualAndHessian() const { return AddCrossBodyTransientConstraintResidualAndHessian; }
		TArray<FGSCrossBodyResidualHessianFunc>& CrossBodyTransientConstraintResidualAndHessian() { return AddCrossBodyTransientConstraintResidualAndHessian; }

		const TArray<FGSPerNodeHessianFunc>& PerNodeHessian() const { return AddPerNodeHessian; }
		TArray<FGSPerNodeHessianFunc>& PerNodeHessian() { return AddPerNodeHessian; }
		const TArray<FGSPerNodeHessianFunc>& TransientPerNodeHessian() const { return AddTransientPerNodeHessian; }
		TArray<FGSPerNodeHessianFunc>& TransientPerNodeHessian() { return AddTransientPerNodeHessian; }
		const TArray<FGSForceDifferentialFunc>& InternalForceDifferentials() const { return AddInternalForceDifferentials; }
		TArray<FGSForceDifferentialFunc>& InternalForceDifferentials() { return AddInternalForceDifferentials; }

		CHAOS_API int32 AddStaticConstraintResidualAndHessianRange(int32 NumConstraints);

		CHAOS_API int32 AddTransientConstraintResidualAndHessianRange(int32 NumConstraints);

		CHAOS_API int32 AddDynamicConstraintResidualAndHessianRange(int32 NumConstraints);

		CHAOS_API int32 AddCrossBodyStaticConstraintResidualAndHessianRange(int32 NumConstraints);

		CHAOS_API int32 AddCrossBodyTransientConstraintResidualAndHessianRange(int32 NumConstraints);

		CHAOS_API int32 AddPerNodeHessianRange(int32 NumConstraints);

		CHAOS_API int32 AddTransientPerNodeHessianRange(int32 NumConstraints);

		CHAOS_API int32 AddAddInternalForceDifferentialsRange(int32 NumConstraints);

		CHAOS_API void AddStaticConstraints(const TArray<TArray<int32>>& ExtraConstraints, const TArray<TArray<int32>>& ExtraIncidentElements, const TArray<TArray<int32>>& ExtraIncidentElementsLocal);

		CHAOS_API void AddTransientConstraints(const TArray<TArray<int32>>& ExtraConstraints, const TArray<TArray<int32>>& ExtraIncidentElements, const TArray<TArray<int32>>& ExtraIncidentElementsLocal, bool CheckIncidentElements = false);

		CHAOS_API void AddDynamicConstraints(const TArray<TArray<int32>>& ExtraConstraints, const TArray<TArray<int32>>& ExtraIncidentElements, const TArray<TArray<int32>>& ExtraIncidentElementsLocal, bool CheckIncidentElements = false);

		CHAOS_API void ResetDynamicConstraints();
		CHAOS_API void ResetTransientConstraints();
		CHAOS_API void ResetTransientConstraintsAndRules();

		CHAOS_API void Apply(FSolverParticlesRange& Particles, const FSolverReal Dt, const int32 MaxWriteIters = 10, const bool Write2File = false, const TPBDActiveView<FSolverParticles>* InParticleActiveView = nullptr);
		CHAOS_API void Apply(FSolverParticles& Particles, const FSolverReal Dt, const int32 MaxWriteIters = 10, const bool Write2File = false, const TPBDActiveView<FSolverParticles>* InParticleActiveView = nullptr);

		CHAOS_API void InitStaticColor(const FSolverParticlesRange& Particles, const TPBDActiveView<FSolverParticles>* InParticleActiveView = nullptr);
		CHAOS_API void InitStaticColor(const FSolverParticles& Particles, const TPBDActiveView<FSolverParticles>* InParticleActiveView = nullptr);

		CHAOS_API void InitTransientColor(const FSolverParticlesRange& Particles);
		CHAOS_API void InitTransientColor(const FSolverParticles& Particles);

		CHAOS_API void InitDynamicColor(const FSolverParticlesRange& Particles);
		CHAOS_API void InitDynamicColor(const FSolverParticles& Particles);

		CHAOS_API void Init(const FSolverReal Dt, const FSolverParticlesRange& Particles);
		CHAOS_API void Init(const FSolverReal Dt, const FSolverParticles& Particles);

		CHAOS_API TArray<TVec3<FSolverReal>> ComputeNewtonResiduals(const FSolverParticlesRange& Particles, const FSolverReal Dt, const bool Write2File = false, TArray<PMatrix<FSolverReal, 3, 3>>* AllParticleHessian = nullptr);
		CHAOS_API TArray<TVec3<FSolverReal>> ComputeNewtonResiduals(const FSolverParticles& Particles, const FSolverReal Dt, const bool Write2File = false, TArray<PMatrix<FSolverReal, 3, 3>>* AllParticleHessian = nullptr);

		CHAOS_API void ApplyCG(FSolverParticlesRange& Particles, const FSolverReal Dt);
		CHAOS_API void ApplyCG(FSolverParticles& Particles, const FSolverReal Dt);

		/* Adds external acceleration, e.g. gravity (0,0,-980) cm/s^2 */
		void AddExternalAcceleration(const TVec3<FSolverReal>& Acceleration) { ExternalAcceleration += Acceleration; };

		void ResetExternalAcceleration() { ExternalAcceleration = TVec3<FSolverReal>((FSolverReal)0.); };

		UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(UseGaussSeidelConstraints, bool);
		UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(AccelerateSolverUsingSOR, bool);
		UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(OmegaSOR, float);
		UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(DoQuasistatics, bool);

	private:

		bool ApplySingleParticle(const int32 ParticleIdx, const FSolverReal Dt, FSolverParticlesRange& Particles);

		void InitializeLambdas();

		static bool IsClean(const TArray<TArray<int32>>& ConstraintsIn, const TArray<TArray<int32>>& IncidentElementsIn, const TArray<TArray<int32>>& IncidentElementsLocalIn);


		//Constraints storage:
		TArray<TArray<int32>> StaticConstraints = {};
		TArray<TArray<int32>> StaticIncidentElements;
		TArray<TArray<int32>> StaticIncidentElementsLocal;
		TArray<TArray<int32>> TransientConstraints = {};
		TArray<TArray<int32>> TransientIncidentElements;
		TArray<TArray<int32>> TransientIncidentElementsLocal;
		TArray<TArray<int32>> DynamicConstraints = {};
		TArray<TArray<int32>> DynamicIncidentElements;
		TArray<TArray<int32>> DynamicIncidentElementsLocal;

		//Lambdas for specifying residual/hessian computations:
		FGSInitialResidualHessianFunc ComputeInitialResidualAndHessian;
		TArray<FGSConstraintResidualHessianFunc> AddStaticConstraintResidualAndHessian;
		TArray<FGSConstraintResidualHessianFunc> AddDynamicConstraintResidualAndHessian;
		TArray<FGSConstraintResidualHessianFunc> AddTransientConstraintResidualAndHessian;
		TArray<FGSPerNodeHessianFunc> AddPerNodeHessian;
		TArray<FGSPerNodeHessianFunc> AddTransientPerNodeHessian;

		TArray<FGSCrossBodyResidualHessianFunc> AddCrossBodyStaticConstraintResidualAndHessian;
		TArray<FGSCrossBodyResidualHessianFunc> AddCrossBodyTransientConstraintResidualAndHessian;

		//Coloring information:
		TArray<int32> StaticParticleColors;
		TArray<TArray<int32>> StaticParticlesPerColor;

		TArray<int32> ParticleColors;
		TArray<TArray<int32>> ParticlesPerColor;

		TArray<int32> StaticIncidentElementsOffsets;
		TArray<int32> TransientIncidentElementsOffsets;
		TArray<int32> DynamicIncidentElementsOffsets;

		bool bDoQuasistatics = false;
		mutable TArray<Chaos::TVector<FSolverReal, 3>> xtilde;

		//SOR variables:
		TFunction<void(FSolverParticlesRange&, int32)> AccelerationTechniquePerParticle;
		mutable TArray<Chaos::TVector<FSolverReal, 3>> X_k_1;
		mutable TArray<Chaos::TVector<FSolverReal, 3>> X_k;
		int32 CurrentIt = 0;
		bool bDoAcceleration = true;
		FSolverReal OmegaSOR = FSolverReal(1.6);
		int32 SORStart = 1; //1 is smallest

		int32 ParallelMax = 1000;

		FDeformableXPBDCorotatedParams CorotatedParams;

		//Newton solver variables:
		TArray<FGSForceDifferentialFunc> AddInternalForceDifferentials;
		TUniquePtr<TArray<int32>> use_list;

		int32 NumTotalParticles;
		TArray<TVec3<FSolverReal>> ReorderedPs;

		FSolverReal MaxDxSize = FSolverReal(UE_MAX_FLT);

		public:
		bool DebugResidual = false;
		bool IsFirstFrame = true;
		int32 PassedIters = 0;

		TVec3<FSolverReal> ExternalAcceleration = TVec3<FSolverReal>((FSolverReal)0.);

	};

	template <typename T, typename ParticleType>
	using FGaussSeidelMainConstraint UE_DEPRECATED(5.8, "Deprecated. this class is to be deleted, use FGaussSeidelMainConstraints instead") = FGaussSeidelMainConstraints;

}

