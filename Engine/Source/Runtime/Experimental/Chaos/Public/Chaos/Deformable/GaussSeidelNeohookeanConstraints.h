// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/PBDSpringConstraintsBase.h"
#include "Chaos/Deformable/GaussSeidelCorotatedConstraints.h"
#include "ChaosStats.h"
#include "Chaos/ImplicitQRSVD.h"
#include "Chaos/GraphColoring.h"
#include "Chaos/NewtonCorotatedCache.h"
#include "Chaos/NeohookeanModel.h"
#include "Chaos/Framework/Parallel.h"


namespace Chaos::Softs
{
	// Tetrahedral constraint using Neohookean constitutive model
	// for position-based nonlinear Gauss Seidel solver [Chen et al., 2024]
	class FGaussSeidelNeohookeanTetrahedralConstraints : public FGaussSeidelCorotatedTetrahedralConstraints
	{
		typedef FGaussSeidelCorotatedTetrahedralConstraints GSBase;
		using GSBase::ParticlesPerColor;
		using GSBase::IncidentElements;
		using GSBase::IncidentElementsLocal;
		using GSBase::ParticleStartIndex;
		using GSBase::LocalNewtonTol;
		using GSBase::bDoQuasistatics;
		using GSBase::ComputeStress;
		using GSBase::ComputeHessianHelper;
		typedef FXPBDCorotatedTetrahedralConstraints Base;
		using Base::MeshConstraints;
		using Base::LambdaArray;
		using Base::Measure;
		using Base::DmInverse;
		using Base::MuElementArray;
		using Base::LambdaElementArray;
		using Base::CorotatedParams;

	public:
		FGaussSeidelNeohookeanTetrahedralConstraints(
			const FSolverParticles& InParticles,
			const TArray<TVector<int32, 4>>& InMesh,
			const TArray<FSolverReal>& EMeshArray,
			const TArray<FSolverReal>& NuMeshArray,
			TArray<FSolverReal>&& AlphaJMeshArray,
			TArray<TArray<int32>>&& IncidentElementsIn,
			TArray<TArray<int32>>&& IncidentElementsLocalIn,
			const int32 ParticleStartIndexIn,
			const int32 ParticleEndIndexIn,
			const bool bDoQuasistaticsIn = false,
			const bool bDoSORIn = true,
			const FSolverReal InOmegaSOR = (FSolverReal)1.6,
			const FDeformableXPBDCorotatedParams& InParams = FDeformableXPBDCorotatedParams(),
			const FSolverReal& NuMesh = (FSolverReal).3,
			const bool bRecordMetricIn = false
		)
			: GSBase(InParticles, InMesh, EMeshArray, NuMeshArray, MoveTemp(AlphaJMeshArray), MoveTemp(IncidentElementsIn), MoveTemp(IncidentElementsLocalIn), ParticleStartIndexIn, ParticleEndIndexIn, bDoQuasistaticsIn, bDoSORIn, InOmegaSOR, InParams, NuMesh, bRecordMetricIn)
		{
			InitializeNeohookeanLambdas();
		}

		virtual ~FGaussSeidelNeohookeanTetrahedralConstraints() = default;

	protected:

		void InitializeNeohookeanLambdas() 
		{
			ComputeStress = [](const Chaos::PMatrix<FSolverReal, 3, 3>& Fe, const FSolverReal mu, const FSolverReal lambda, Chaos::PMatrix<FSolverReal, 3, 3>& P)
			{
				PNeohookeanMM(Fe, mu, lambda, P);
			};
			ComputeHessianHelper = [](const Chaos::PMatrix<FSolverReal, 3, 3>& Fe, const Chaos::PMatrix<FSolverReal, 3, 3>& DmInv, const FSolverReal mu, const FSolverReal lambda, const int32 local_index, const FSolverReal Coeff, Chaos::PMatrix<FSolverReal, 3, 3>& final_hessian)
			{
				FSolverReal LambdaHat = mu + lambda;
				Chaos::PMatrix<FSolverReal, 3, 3> JFinvT((FSolverReal)0.);
				JFinvT.SetAt(0, 0, Fe.GetAt(1, 1) * Fe.GetAt(2, 2) - Fe.GetAt(2, 1) * Fe.GetAt(1, 2));
				JFinvT.SetAt(0, 1, Fe.GetAt(2, 0) * Fe.GetAt(1, 2) - Fe.GetAt(1, 0) * Fe.GetAt(2, 2));
				JFinvT.SetAt(0, 2, Fe.GetAt(1, 0) * Fe.GetAt(2, 1) - Fe.GetAt(2, 0) * Fe.GetAt(1, 1));
				JFinvT.SetAt(1, 0, Fe.GetAt(2, 1) * Fe.GetAt(0, 2) - Fe.GetAt(0, 1) * Fe.GetAt(2, 2));
				JFinvT.SetAt(1, 1, Fe.GetAt(0, 0) * Fe.GetAt(2, 2) - Fe.GetAt(2, 0) * Fe.GetAt(0, 2));
				JFinvT.SetAt(1, 2, Fe.GetAt(2, 0) * Fe.GetAt(0, 1) - Fe.GetAt(0, 0) * Fe.GetAt(2, 1));
				JFinvT.SetAt(2, 0, Fe.GetAt(0, 1) * Fe.GetAt(1, 2) - Fe.GetAt(1, 1) * Fe.GetAt(0, 2));
				JFinvT.SetAt(2, 1, Fe.GetAt(1, 0) * Fe.GetAt(0, 2) - Fe.GetAt(0, 0) * Fe.GetAt(1, 2));
				JFinvT.SetAt(2, 2, Fe.GetAt(0, 0) * Fe.GetAt(1, 1) - Fe.GetAt(1, 0) * Fe.GetAt(0, 1));

				Chaos::PMatrix<FSolverReal, 3, 3> JFinv = JFinvT.GetTransposed();

				if (local_index == 0) {
					FSolverReal DmInvsum = FSolverReal(0);
					for (int32 nu = 0; nu < 3; nu++) {
						FSolverReal localDmsum = FSolverReal(0);
						for (int32 k = 0; k < 3; k++) {
							localDmsum += DmInv.GetAt(k, nu);
						}
						DmInvsum += localDmsum * localDmsum;
					}
					for (int32 alpha = 0; alpha < 3; alpha++) {
						final_hessian.SetAt(alpha, alpha, final_hessian.GetAt(alpha, alpha) + Coeff * mu * DmInvsum);
					}

					Chaos::PMatrix<FSolverReal, 3, 3> DmInvJFinv = JFinv * DmInv;
					Chaos::TVector<FSolverReal, 3> l((FSolverReal)0.);
					for (int32 alpha = 0; alpha < 3; alpha++) {
						for (int32 k = 0; k < 3; k++) {
							l[alpha] += DmInvJFinv.GetAt(k, alpha);
						}
					}
					for (int32 alpha = 0; alpha < 3; alpha++)
					{
						final_hessian.SetRow(alpha, final_hessian.GetRow(alpha) + Coeff * LambdaHat * l[alpha] * l);
					}

				}
				else {
					FSolverReal DmInvsum = FSolverReal(0);
					for (int32 nu = 0; nu < 3; nu++)
					{
						DmInvsum += DmInv.GetAt(local_index - 1, nu) * DmInv.GetAt(local_index - 1, nu);
					}
					for (int32 alpha = 0; alpha < 3; alpha++)
					{
						final_hessian.SetAt(alpha, alpha, final_hessian.GetAt(alpha, alpha) + Coeff * mu * DmInvsum);
					}

					Chaos::PMatrix<FSolverReal, 3, 3> DmInvJFinv = JFinv * DmInv;
					Chaos::TVector<FSolverReal, 3> l((FSolverReal)0.);
					for (int32 alpha = 0; alpha < 3; alpha++) {
						l[alpha] = DmInvJFinv.GetAt(local_index - 1, alpha);
					}
					for (int32 alpha = 0; alpha < 3; alpha++)
					{
						final_hessian.SetRow(alpha, final_hessian.GetRow(alpha) + Coeff * LambdaHat * l[alpha] * l);
					}
				}
			};
		}

	};

	template <typename T, typename ParticleType>
	using FGaussSeidelNeohookeanConstraints UE_DEPRECATED(5.8, "Deprecated. this class is to be deleted, use FGaussSeidelNeohookeanTetrahedralConstraints instead") = FGaussSeidelNeohookeanTetrahedralConstraints;

}// End namespace Chaos::Softs
