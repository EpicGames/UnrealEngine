// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/BoundingVolumeHierarchy.h"
#include "Chaos/PBDSpringConstraintsBase.h"
#include "Chaos/XPBDCorotatedConstraints.h"
#include "ChaosStats.h"
#include "Chaos/ImplicitQRSVD.h"
#include "Chaos/GraphColoring.h"
#include "Chaos/NewtonCorotatedCache.h"
#include "Chaos/Framework/Parallel.h"
#include "Chaos/Utilities.h"
#include "Chaos/DebugDrawQueue.h"
#include "Chaos/XPBDWeakConstraints.h"
#include "Chaos/Triangle.h"
#include "Chaos/TriangleCollisionPoint.h"
#include "Chaos/TriangleMesh.h"
#include <unordered_map>
#include "Chaos/HierarchicalSpatialHash.h"
#include "Chaos/PBDSelfCollisionSphereConstraints.h"

namespace Chaos::Softs
{
	using Chaos::TVec3;

	class FGaussSeidelParticleSphereRepulsionConstraints
	{
	public:
		CHAOS_API FGaussSeidelParticleSphereRepulsionConstraints(FSolverReal InRadius, FSolverReal InStiffness, const FSolverParticles& InParticles, const FDeformableXPBDWeightedSpringConstraintParams& InParams);

		virtual ~FGaussSeidelParticleSphereRepulsionConstraints() = default;

		CHAOS_API void AddSphereRepulsionResidualAndHessian(const FSolverParticlesRange& InParticles, const int32 ConstraintIndex, const int32 LocalIndex, const FSolverReal Dt, TVec3<FSolverReal>& ParticleResidual, Chaos::PMatrix<FSolverReal, 3, 3>& ParticleHessian);

		CHAOS_API void VisualizeAllBindings(const FSolverParticles& InParticles, const FSolverReal Dt) const;

		CHAOS_API void Init(const FSolverParticles& InParticles, const FSolverReal Dt) const;

		CHAOS_API void UpdateSphereRepulsionConstraints(const FSolverParticles& Particles, const TArray<int32>& SurfaceVertices, const TArray<int32>& ComponentIndex);

		CHAOS_API void ReturnSphereRepulsionConstraints(TArray<TArray<int32>>& ExtraConstraints, TArray<TArray<int32>>& ExtraIncidentElements, TArray<TArray<int32>>& ExtraIncidentElementsLocal);

	protected:
		TArray<TVec2<int32>> Constraints;
		FSolverReal Radius = FSolverReal(0);
		FSolverReal Stiffness = FSolverReal(0);
		TArray<FSolverReal> ConstraintStiffness;
	private:
		struct FSphereSpatialEntry
		{
			const TConstArrayView<FSolverVec3>* Points;
			int32 Index;

			FSolverVec3 X() const
			{
				return (*Points)[Index];
			}

			template<typename TPayloadType>
			int32 GetPayload(int32) const
			{
				return Index;
			}
		};
		TArray<FSolverVec3> ReferencePositions;
		FDeformableXPBDWeightedSpringConstraintParams DebugDrawParams;
	};

	template <typename T, typename ParticleType>
	using FGaussSeidelSphereRepulsionConstraints UE_DEPRECATED(5.8, "Deprecated. this class is to be deleted, use FGaussSeidelParticleSphereRepulsionConstraints instead") = FGaussSeidelParticleSphereRepulsionConstraints;

}// End namespace Chaos::Softs
