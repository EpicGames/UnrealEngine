// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Utilities.h"
#include "Chaos/PBDSoftsEvolutionFwd.h"
#include "Chaos/PBDSoftsSolverParticles.h"

namespace Chaos::Softs
{
	// Tetrahdral volume constraint for position-based nonlinear Gauss Seidel solver [Chen et al., 2024]
	class FGaussSeidelUnilateralVolumeConstraints
	{
	public:
		CHAOS_API FGaussSeidelUnilateralVolumeConstraints(
			const FSolverParticles& Particles,
			TArray<TVector<int32, 4>>&& InConstraints,
			TArray<FSolverReal> InStiffnessArray);

		CHAOS_API void AddEnergy(const FSolverParticles& InParticles, const int32 ConstraintIndex, const FSolverReal Dt, FSolverReal& Energy) const;

		CHAOS_API void AddResidualAndHessian(const FSolverParticles& InParticles, const int32 ConstraintIndex, const int32 LocalIndex, const FSolverReal Dt, TVec3<FSolverReal>& ParticleResidual, Chaos::PMatrix<FSolverReal, 3, 3>& ParticleHessian) const;

		CHAOS_API TArray<TArray<int32>> GetStaticConstraintArrays(const TArray<TArray<int32>>*& OutIncidentElements, const TArray<TArray<int32>>*& OutIncidentElementsLocal) const;

		UE_DEPRECATED(5.8, "Use GetStaticConstraintArrays with pointer output parameters instead.")
		CHAOS_API TArray<TArray<int32>> GetStaticConstraintArrays(TArray<TArray<int32>>& IncidentElements, TArray<TArray<int32>>& IncidentElementsLocal) const;

		int32 NumConstraints() const
		{
			return Constraints.Num();
		}

	private:
		static FSolverReal ComputeVolume(const TVec3<FSolverReal>& P1, const TVec3<FSolverReal>& P2, const TVec3<FSolverReal>& P3, const TVec3<FSolverReal>& P4);

		TArray<TVector<int32, 4>> Constraints;
		TArray<FSolverReal> Volumes;
		TArray<TArray<int32>> IncidentElements;
		TArray<TArray<int32>> IncidentElementsLocal;
		TArray<FSolverReal> StiffnessArray;
	};

	template <typename T, typename ParticleType>
	using FGaussSeidelUnilateralTetConstraints UE_DEPRECATED(5.8, "Deprecated. this class is to be deleted, use FGaussSeidelUnilateralVolumeConstraints instead") = FGaussSeidelUnilateralVolumeConstraints;

}// End namespace Chaos::Softs
