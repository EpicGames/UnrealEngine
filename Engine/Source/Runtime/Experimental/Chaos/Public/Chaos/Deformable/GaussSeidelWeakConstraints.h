// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/BoundingVolumeHierarchy.h"
#include "Chaos/PBDSpringConstraintsBase.h"
#include "Chaos/XPBDCorotatedConstraints.h"
#include "ChaosStats.h"
#include "Chaos/ImplicitQRSVD.h"
#include "Chaos/GraphColoring.h"
#include "Chaos/NewtonCorotatedCache.h"
#include "Chaos/Framework/Parallel.h"
#include "Chaos/Utilities.h"
#include "Chaos/DebugDrawQueue.h"
#include "Chaos/XPBDWeakConstraints.h"
#include "Chaos/Triangle.h"
#include "Chaos/TriangleCollisionPoint.h"
#include "Chaos/TriangleMesh.h"
namespace Chaos::Softs
{
	using Chaos::TVec3;

	struct FGaussSeidelSpringConstraintSingleData
	{
		TArray<int32> SingleIndices  = {};
		TArray<int32> SingleSecondIndices = {};
		FSolverReal SingleStiffness = (FSolverReal)0.;
		TArray<FSolverReal> SingleWeights = {};
		TArray<FSolverReal> SingleSecondWeights = {};
		bool bIsAnisotropic = false;
		TVec3<FSolverReal> SingleNormal = TVec3<FSolverReal>((FSolverReal)0.);
		bool bIsZeroRestLength = false;
		FSolverReal RestLength = (FSolverReal)0.;
	};

	class FGaussSeidelSpringConstraintData : public TArrayCollection
	{
	public:
		CHAOS_API FGaussSeidelSpringConstraintData();
		CHAOS_API FGaussSeidelSpringConstraintData(const FGaussSeidelSpringConstraintData& Other);
		CHAOS_API FGaussSeidelSpringConstraintData(FGaussSeidelSpringConstraintData&& Other);

		virtual ~FGaussSeidelSpringConstraintData() = default;

		void AddConstraints(const int32 Num) { AddElementsHelper(Num); }
		void RemoveConstraint(const int32 Idx) { RemoveAtSwapHelper(Idx); }

		CHAOS_API void SetSingleConstraint(const FGaussSeidelSpringConstraintSingleData& SingleData, const int32 ConstraintIndex);
		CHAOS_API void AddSingleConstraint(const FGaussSeidelSpringConstraintSingleData& SingleData);

		int32 Size() const { return static_cast<int32>(MSize); }
		void Resize(const int32 Num) { ResizeHelper(Num); }

		CHAOS_API FGaussSeidelSpringConstraintData& operator=(FGaussSeidelSpringConstraintData&& Other);

		inline const TArrayCollectionArray<TArray<int32>>& Indices() const { return MIndices; }
		const TArray<int32>& GetIndices(const int32 Index) const { return MIndices[Index]; }
		void SetIndices(const int32 Index, const TArray<int32>& InIndices) { MIndices[Index] = InIndices; }

		inline const TArrayCollectionArray<TArray<int32>>& SecondIndices() const { return MSecondIndices; }
		const TArray<int32>& GetSecondIndices(const int32 Index) const { return MSecondIndices[Index]; }
		void SetSecondIndices(const int32 Index, const TArray<int32>& InIndices) { MSecondIndices[Index] = InIndices; }

		inline const TArrayCollectionArray<TArray<FSolverReal>>& Weights() const { return MWeights; }
		const TArray<FSolverReal>& GetWeights(const int32 Index) const { return MWeights[Index]; }
		void SetWeights(const int32 Index, const TArray<FSolverReal>& InWeights) { MWeights[Index] = InWeights; }

		inline const TArrayCollectionArray<TArray<FSolverReal>>& SecondWeights() const { return MSecondWeights; }
		const TArray<FSolverReal>& GetSecondWeights(const int32 Index) const { return MSecondWeights[Index]; }
		void SetSecondWeights(const int32 Index, const TArray<FSolverReal>& InWeights) { MSecondWeights[Index] = InWeights; }

		const bool GetIsAnisotropic(const int32 Index) const { return MIsAnisotropic[Index]; }
		void SetIsAnisotropic(const int32 Index, const bool InIsAnisotropic) { MIsAnisotropic[Index] = InIsAnisotropic; }

		inline const TArrayCollectionArray<TVec3<FSolverReal>>& Normals() const { return MNormals; }
		const TVec3<FSolverReal>& GetNormal(const int32 Index) const { return MNormals[Index]; }
		void SetNormal(const int32 Index, const TVec3<FSolverReal>& InNormal) { MNormals[Index] = InNormal; }

		inline const TArrayCollectionArray<FSolverReal>& Stiffness() const { return MStiffness; }
		FSolverReal GetStiffness(const int32 Index) const { return MStiffness[Index]; }
		void SetStiffness(const int32 Index, const FSolverReal InStiffness) { MStiffness[Index] = InStiffness; }

		const bool GetIsZeroRestLength(const int32 Index) const { return MIsZeroRestLength[Index]; }
		void SetIsZeroRestLength(const int32 Index, const bool InIsZeroRestLength) { MIsZeroRestLength[Index] = InIsZeroRestLength; }

		void SetRestLength(const int32 Index, const FSolverReal InRestLength) { MRestLength[Index] = InRestLength; }

		CHAOS_API const FGaussSeidelSpringConstraintSingleData GetSingleConstraintData(const int32 ConstraintIndex) const;

	private:
		TArrayCollectionArray<TArray<int32>> MIndices;
		TArrayCollectionArray<TArray<int32>> MSecondIndices;
		TArrayCollectionArray<TArray<FSolverReal>> MWeights;
		TArrayCollectionArray<TArray<FSolverReal>> MSecondWeights;
		TArrayCollectionArray<FSolverReal> MStiffness;
		TArrayCollectionArray<bool> MIsAnisotropic;
		TArrayCollectionArray<TVector<FSolverReal, 3>> MNormals;
		TArrayCollectionArray<bool> MIsZeroRestLength;
		TArrayCollectionArray<FSolverReal> MRestLength;
	};

	// Spring constraint for position-based nonlinear Gauss Seidel solver [Chen et al., 2024]
	struct FGaussSeidelSpringConstraints
	{
	public:
		//TODO(Yizhou): Add unittest for Gauss Seidel Weak Constraints
		CHAOS_API FGaussSeidelSpringConstraints(
			const TArray<TArray<int32>>& InIndices,
			const TArray<TArray<FSolverReal>>& InWeights,
			const TArray<FSolverReal>& InStiffness,
			const TArray<TArray<int32>>& InSecondIndices,
			const TArray<TArray<FSolverReal>>& InSecondWeights,
			const TArray<bool>& InIsAnisotropic,
			const TArray<bool>& InZeroRestLength,
			const FDeformableXPBDWeightedSpringConstraintParams& InParams
		);

		// Scalar-bool overload: applies the same bIsAnisotropic and bIsZeroRestLength to all constraints
		CHAOS_API FGaussSeidelSpringConstraints(
			const TArray<TArray<int32>>& InIndices,
			const TArray<TArray<FSolverReal>>& InWeights,
			const TArray<FSolverReal>& InStiffness,
			const TArray<TArray<int32>>& InSecondIndices,
			const TArray<TArray<FSolverReal>>& InSecondWeights,
			const FDeformableXPBDWeightedSpringConstraintParams& InParams,
			bool bIsAnisotropic,
			bool bIsZeroRestLength
		);

		UE_DEPRECATED(5.8, "Replaced with constructor with IsAnisotropic and IsZeroRestLength.")
		CHAOS_API FGaussSeidelSpringConstraints(
			const TArray<TArray<int32>>& InIndices,
			const TArray<TArray<FSolverReal>>& InWeights,
			const TArray<FSolverReal>& InStiffness,
			const TArray<TArray<int32>>& InSecondIndices,
			const TArray<TArray<FSolverReal>>& InSecondWeights,
			const FDeformableXPBDWeightedSpringConstraintParams& InParams
		);

		struct FGaussSeidelConstraintHandle
		{
			int32 ConstraintIndex;
		};

		virtual ~FGaussSeidelSpringConstraints() = default;

		CHAOS_API void ComputeInitialWCData(const FSolverParticlesRange& InParticles);
		CHAOS_API void ComputeInitialWCData(const FSolverParticles& InParticles);

		CHAOS_API void AddWCHessian(const int32 ParticleIdx, const FSolverReal Dt, Chaos::PMatrix<FSolverReal, 3, 3>& ParticleHessian) const;

		void AddExtraConstraints(const TArray<TArray<int32>>& InIndices,
								const TArray<TArray<FSolverReal>>& InWeights,
								const TArray<FSolverReal>& InStiffness,
								const TArray<TArray<int32>>& InSecondIndices,
								const TArray<TArray<FSolverReal>>& InSecondWeights,
								const TArray<bool>& InIsAnisotropic,
								const TArray<bool>& InIsZeroRestLength);

		CHAOS_API void Init(const FSolverParticlesRange& InParticles, const FSolverReal Dt);
		CHAOS_API void Init(const FSolverParticles& InParticles, const FSolverReal Dt);

		//CollisionDetectionSpatialHash should be faster than CollisionDetectionBVH
		void CollisionDetectionBVH(const FSolverParticles& Particles, const TArray<TVec3<int32>>& SurfaceElements, const TArray<int32>& ComponentIndex, float DetectRadius = 1.f, float PositionTargetStiffness = 10000.f, bool UseAnisotropicSpring = true);

		template<typename SpatialAccelerator>
		void CollisionDetectionSpatialHash(const FSolverParticles& Particles, const TArray<int32>& SurfaceVertices, const FTriangleMesh& TriangleMesh, const TArray<int32>& ComponentIndex, const SpatialAccelerator& Spatial, float DetectRadius = 1.f, float PositionTargetStiffness = 10000.f, bool UseAnisotropicSpring = true);

		template<typename SpatialAccelerator>
		void CollisionDetectionSpatialHashInComponent(const FSolverParticles& Particles, const TArray<int32>& SurfaceVertices, const FTriangleMesh& TriangleMesh, const TMap<int32, TSet<int32>>& ExcludeMap, const SpatialAccelerator& Spatial, float DetectRadius = 0.f, float PositionTargetStiffness = 10000.f, bool UseAnisotropicSpring = true);

		CHAOS_API void ComputeCollisionWCDataSimplified(TArray<TArray<int32>>& ExtraConstraints, TArray<TArray<int32>>& ExtraWCIncidentElements, TArray<TArray<int32>>& ExtraWCIncidentElementsLocal);

		CHAOS_API const TArray<TArray<int32>>& GetStaticConstraintArrays(const TArray<TArray<int32>>*& OutIncidentElements, const TArray<TArray<int32>>*& OutIncidentElementsLocal) const;

		UE_DEPRECATED(5.8, "Use GetStaticConstraintArrays with pointer output parameters instead.")
		CHAOS_API const TArray<TArray<int32>>& GetStaticConstraintArrays(TArray<TArray<int32>>& IncidentElements, TArray<TArray<int32>>& IncidentElementsLocal) const;

		CHAOS_API TArray<TArray<int32>> GetDynamicConstraintArrays(TArray<TArray<int32>>& IncidentElements, TArray<TArray<int32>>& IncidentElementsLocal) const;

		UE_DEPRECATED(5.8, "Replaced with AddWCResidual for more general cases.")
		void AddZeroRestLengthWCResidualAndHessian(const FSolverParticles& InParticles, const int32 ConstraintIndex, const int32 LocalIndex, const FSolverReal Dt, TVec3<FSolverReal>& ParticleResidual, Chaos::PMatrix<FSolverReal, 3, 3>& ParticleHessian) const;

		CHAOS_API void AddWCResidual(const FSolverParticlesRange& InParticles, const int32 ConstraintIndex, const int32 LocalIndex, const FSolverReal Dt, TVec3<FSolverReal>& ParticleResidual, Chaos::PMatrix<FSolverReal, 3, 3>& ParticleHessian) const;

		CHAOS_API void AddWCResidual(const FSolverParticles& InParticles, const int32 ConstraintIndex, const int32 LocalIndex, const FSolverReal Dt, TVec3<FSolverReal>& ParticleResidual, Chaos::PMatrix<FSolverReal, 3, 3>& ParticleHessian) const;

		int32 GetInitialWCSize() const { return InitialWCSize; }
		const FGaussSeidelSpringConstraintData& GetConstraintsData() const { return ConstraintsData; }

	protected:
		CHAOS_API virtual void SetConstraintsRestLength(const FSolverParticlesRange& InParticles, const FSolverParticlesRange& InSecondParticles);

		CHAOS_API void UpdateTriangleNormalAndNodalWeight(const FSolverParticlesRange& InParticles, const FSolverParticlesRange& InSecondParticles, bool bUseParticleX);

		CHAOS_API TVec3<FSolverReal> ComputeSpringEdge(const FSolverParticlesRange& InParticles, const FSolverParticlesRange& InSecondParticles,
			const TArray<int32>& LocalIndices, const TArray<int32>& LocalSecondIndices,
			const TArray<FSolverReal>& Weight, const TArray<FSolverReal>& SecondWeight, bool bUseParticleX) const;

		CHAOS_API TVec3<FSolverReal> ComputeSpringEdge(const FSolverParticles& InParticles, const FSolverParticles& InSecondParticles,
			const TArray<int32>& LocalIndices, const TArray<int32>& LocalSecondIndices,
			const TArray<FSolverReal>& Weight, const TArray<FSolverReal>& SecondWeight, bool bUseParticleX) const;

		CHAOS_API TVec3<FSolverReal> ComputeSpringEdge(const FSolverParticlesRange& InParticles,
			const TArray<int32>& LocalIndices, const TArray<int32>& LocalSecondIndices,
			const TArray<FSolverReal>& Weight, const TArray<FSolverReal>& SecondWeight, bool bUseParticleX) const;

		CHAOS_API TVec3<FSolverReal> ComputeSpringEdge(const FSolverParticles& InParticles, const TArray<int32>& LocalIndices,
			const TArray<int32>& LocalSecondIndices, const TArray<FSolverReal>& Weight, const TArray<FSolverReal>& SecondWeight, bool bUseParticleX) const;

		CHAOS_API virtual void SetNoCollisionData();

		FGaussSeidelSpringConstraintData ConstraintsData;
		TArray<TArray<FSolverReal>> NodalWeights;
		TArray<TArray<int32>> WCIncidentElements;
		TArray<TArray<int32>> WCIncidentElementsLocal;

		// Static constraint data (before adding collision constraints)
		TArray<TArray<FSolverReal>> NoCollisionNodalWeights;
		TArray<TArray<int32>> NoCollisionConstraints;
		TArray<TArray<int32>> NoCollisionWCIncidentElements;
		TArray<TArray<int32>> NoCollisionWCIncidentElementsLocal;

	private:
		CHAOS_API virtual void ComputeIncidentElements();

		void UpdateTriangleNormal(const FSolverParticlesRange& InParticles, const FSolverParticlesRange& InSecondParticles, bool bUseParticleX);

		CHAOS_API virtual void UpdateNodalWeight(const FSolverParticlesRange& InParticles, const FSolverParticlesRange& InSecondParticles, bool bUseParticleX);

		void UpdateTriangleNormalAndNodalWeight(const FSolverParticlesRange& InParticles, bool bUseParticleX);

		void Resize(int32 Size);

		void UpdatePointTriangleCollisionWCData(const FSolverParticles& Particles);

		CHAOS_API void VisualizeAllBindings(const FSolverParticlesRange& InParticles, const FSolverReal Dt) const;

		int32 InitialWCSize;
		FDeformableXPBDWeightedSpringConstraintParams DebugDrawParams;
	};

	template <typename T>
	using FGaussSeidelWeakConstraintSingleData UE_DEPRECATED(5.8, "Deprecated. this struct is to be deleted, use FGaussSeidelSpringConstraintSingleData instead") = FGaussSeidelSpringConstraintSingleData;

	template <class T>
	using FGaussSeidelWeakConstraintData UE_DEPRECATED(5.8, "Deprecated. this class is to be deleted, use FGaussSeidelSpringConstraintData instead") = FGaussSeidelSpringConstraintData;

	template <typename T, typename ParticleType>
	using FGaussSeidelWeakConstraints UE_DEPRECATED(5.8, "Deprecated. this struct is to be deleted, use FGaussSeidelSpringConstraints instead") = FGaussSeidelSpringConstraints;

}// End namespace Chaos::Softs
