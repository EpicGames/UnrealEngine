// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Curve.h"
#include "Chaos/Framework/Parallel.h"
#include "Chaos/XPBDCorotatedConstraints.h"
#include "GeometryCollection/Facades/CollectionMuscleActivationFacade.h"

DEFINE_LOG_CATEGORY_STATIC(LogMuscleActivationConstraint, Log, All);

namespace Chaos::Softs
{
	using Chaos::TVec3;
	typedef GeometryCollection::Facades::FMuscleActivationFacade Facade;
	typedef GeometryCollection::Facades::FMuscleActivationData Data;

	// Stores data for muscle activation and apply rest volume changes to tetrahedral constraints
	class FMuscleActivationTetrahedralConstraints
	{
	public:
		FMuscleActivationTetrahedralConstraints(){}

		virtual ~FMuscleActivationTetrahedralConstraints() = default;

		void AddMuscles(const FSolverParticles& RestParticles, const Facade& FMuscleActivation, int32 VertexOffset = 0, int32 ElementOffset = 0)
		{
			for (int MuscleIdx = 0; MuscleIdx < FMuscleActivation.NumMuscles(); MuscleIdx++)
			{
				Data MuscleActivationData = FMuscleActivation.GetMuscleActivationData(MuscleIdx);
				if (FMuscleActivation.IsValidGeometryIndex(MuscleActivationData.GeometryGroupIndex))
				{
					if (ensureMsgf(MuscleActivationData.MuscleLengthRatioThresholdForMaxActivation > 0 &&
						MuscleActivationData.MuscleLengthRatioThresholdForMaxActivation < 1,
						TEXT("MuscleLengthRatioThresholdForMaxActivation %f of muscle indexed %d is out of range (0,1), please check your setup."), MuscleActivationData.MuscleLengthRatioThresholdForMaxActivation, MuscleIdx)
						&&
						ensureMsgf(MuscleActivationData.MuscleActivationElement.Num() == MuscleActivationData.FiberDirectionMatrix.Num(),
							TEXT("MuscleActivationElement size %d is not equal to FiberDirectionMatrix size %d for muscle indexed %d"), MuscleActivationData.MuscleActivationElement.Num(), MuscleActivationData.FiberDirectionMatrix.Num(), MuscleIdx)
						&&
						ensureMsgf(MuscleActivationData.FiberLengthRatioAtMaxActivation > 0 &&
							MuscleActivationData.FiberLengthRatioAtMaxActivation < 1,
							TEXT("FiberLengthRatioAtMaxActivation %f of muscle indexed %d is out of range (0,1), please check your setup."), MuscleActivationData.FiberLengthRatioAtMaxActivation, MuscleIdx)
						&&
						ensureMsgf(MuscleActivationData.InflationVolumeScale > 0,
							TEXT("FiberLengthRatioAtMaxActivation %f of muscle indexed %d is <= 0, please check your setup."), MuscleActivationData.InflationVolumeScale, MuscleIdx))
					{
						MuscleActivationData.OriginInsertionPair[0] += VertexOffset;
						MuscleActivationData.OriginInsertionPair[1] += VertexOffset;
						if (!RestParticles.XArray().IsValidIndex(MuscleActivationData.OriginInsertionPair[0]) ||
							!RestParticles.XArray().IsValidIndex(MuscleActivationData.OriginInsertionPair[1]))
						{
							UE_LOGF(LogMuscleActivationConstraint, Error, "Muscle Idx[%d] has invalid origin[%d] or insertion[%d].",
								MuscleIdx, MuscleActivationData.OriginInsertionPair[0], MuscleActivationData.OriginInsertionPair[1]);
							continue;
						}
						const int32 OldSize = MuscleActivationElement.AddDefaulted(1);
						FiberDirectionMatrix.AddDefaulted(1);
						ContractionVolumeScale.AddDefaulted(1);
						for (int32 i = 0; i < MuscleActivationData.MuscleActivationElement.Num(); i++)
						{
							if (FMuscleActivation.IsValidElementIndex(MuscleActivationData.MuscleActivationElement[i]))
							{
								MuscleActivationElement[OldSize].Add(MuscleActivationData.MuscleActivationElement[i] + ElementOffset);
								FiberDirectionMatrix[OldSize].Add(MuscleActivationData.FiberDirectionMatrix[i]);
								ContractionVolumeScale[OldSize].Add(MuscleActivationData.ContractionVolumeScale[i]);
							}
						}
						FiberLengthRatioAtMaxActivation.Add(MuscleActivationData.FiberLengthRatioAtMaxActivation);
						MuscleLengthRatioThresholdForMaxActivation.Add(MuscleActivationData.MuscleLengthRatioThresholdForMaxActivation);
						InflationVolumeScale.Add(MuscleActivationData.InflationVolumeScale);
						OriginInsertionPair.Add(MuscleActivationData.OriginInsertionPair);
						MuscleRestLength.Add((RestParticles.GetX(MuscleActivationData.OriginInsertionPair[0]) - RestParticles.GetX(MuscleActivationData.OriginInsertionPair[1])).Size());
						MuscleActivation.Add(FSolverReal(0));
						MuscleVertexOffset.Add(FMuscleActivation.MuscleVertexOffset(MuscleIdx));
						MuscleVertexCount.Add(FMuscleActivation.NumMuscleVertices(MuscleIdx));
						LengthActivationCurves.Add(FMuscleActivation.GetLengthActivationCurve(MuscleIdx));
					}
				}
				else
				{
					UE_LOGF(LogMuscleActivationConstraint, Error, "Muscle Idx[%d] has invalid geometry index[%d].", 
						MuscleIdx, MuscleActivationData.GeometryGroupIndex);
				}
			}
		}

		void UpdateLengthBasedMuscleActivation(const FSolverParticles& InParticles)
		{
			for (int32 MuscleIdx = 0; MuscleIdx < MuscleActivationElement.Num(); MuscleIdx++)
			{
				if (ensureMsgf(MuscleLengthRatioThresholdForMaxActivation[MuscleIdx] > 0 && 
					MuscleLengthRatioThresholdForMaxActivation[MuscleIdx] < 1,
					TEXT("MuscleLengthRatioThresholdForMaxActivation %f of muscle indexed %d is out of range (0,1), please check your setup."), MuscleLengthRatioThresholdForMaxActivation[MuscleIdx], MuscleIdx))
				{
					// calculate origin/insertion length
					const FSolverReal MuscleLengthRatio = (InParticles.P(OriginInsertionPair[MuscleIdx][0]) - InParticles.P(OriginInsertionPair[MuscleIdx][1])).Size() / MuscleRestLength[MuscleIdx];
					if (MuscleLengthRatio >= FSolverReal(1))
					{
						// not active
						MuscleActivation[MuscleIdx] = FSolverReal(0);
					}
					else
					{
						FSolverReal NormalizedMuscleLengthLevel = FMath::Clamp((FSolverReal(1) - MuscleLengthRatio) / (FSolverReal(1) - MuscleLengthRatioThresholdForMaxActivation[MuscleIdx]), FSolverReal(0), FSolverReal(1));
						if (LengthActivationCurves[MuscleIdx].GetNumKeys())
						{
							MuscleActivation[MuscleIdx] = LengthActivationCurves[MuscleIdx].Eval(NormalizedMuscleLengthLevel);
						}
						else
						{
							// no keys, default to linear muscle activation model: muscle reaches max activation 1 at threshold length
							MuscleActivation[MuscleIdx] = NormalizedMuscleLengthLevel;
						}
					}
				}
			}
		}

		void ApplyMuscleActivation(FXPBDCorotatedTetrahedralConstraints& Constraints) const
		{
			for (int32 MuscleIdx = 0; MuscleIdx < MuscleActivationElement.Num(); MuscleIdx++)
			{
				if (ensureMsgf(MuscleActivationElement[MuscleIdx].Num() == FiberDirectionMatrix[MuscleIdx].Num(), 
					TEXT("MuscleActivationElement[%d].Num() = %d, not equal to FiberDirectionMatrix[%d].Num() = %d"), MuscleIdx, MuscleActivationElement[MuscleIdx].Num(), MuscleIdx, FiberDirectionMatrix[MuscleIdx].Num())
					&& ensureMsgf(FiberLengthRatioAtMaxActivation[MuscleIdx] > 0 &&
						FiberLengthRatioAtMaxActivation[MuscleIdx] < 1,
						TEXT("FiberLengthRatioAtMaxActivation %f of muscle indexed %d is out of range (0,1), please check your setup."), FiberLengthRatioAtMaxActivation[MuscleIdx], MuscleIdx))
				{
					const FSolverReal FiberLengthRatio = FSolverReal(1) - MuscleActivation[MuscleIdx] * (FSolverReal(1) - FiberLengthRatioAtMaxActivation[MuscleIdx]);
					for (int32 ElemIdx = 0; ElemIdx < MuscleActivationElement[MuscleIdx].Num(); ElemIdx++)
					{
						Constraints.ModifyDmInverseFromMuscleLength(MuscleActivationElement[MuscleIdx][ElemIdx], FiberLengthRatio, FiberDirectionMatrix[MuscleIdx][ElemIdx], ContractionVolumeScale[MuscleIdx][ElemIdx]);
					}
				}
			}
		}

		void ApplyInflationVolumeScale(FXPBDCorotatedTetrahedralConstraints& Constraints) const
		{
			for (int32 MuscleIdx = 0; MuscleIdx < MuscleActivationElement.Num(); MuscleIdx++)
			{
				if (ensureMsgf(MuscleActivationElement[MuscleIdx].Num() == FiberDirectionMatrix[MuscleIdx].Num(),
					TEXT("MuscleActivationElement[%d].Num() = %d, not equal to FiberDirectionMatrix[%d].Num() = %d"), MuscleIdx, MuscleActivationElement[MuscleIdx].Num(), MuscleIdx, FiberDirectionMatrix[MuscleIdx].Num())
					&& ensureMsgf(InflationVolumeScale[MuscleIdx] > 0,
						TEXT("InflationVolumeScale %f of muscle indexed %d is <= 0, please check your setup."), InflationVolumeScale[MuscleIdx], MuscleIdx))
				{
					for (int32 ElemIdx = 0; ElemIdx < MuscleActivationElement[MuscleIdx].Num(); ElemIdx++)
					{
						Constraints.ModifyDmInverseSaveFromInflationVolumeScale(MuscleActivationElement[MuscleIdx][ElemIdx], InflationVolumeScale[MuscleIdx], FiberDirectionMatrix[MuscleIdx][ElemIdx]);
					}
				}
			}
		}

		int32 NumMuscles() { return MuscleActivationElement.Num(); };
		int32 GetMuscleVertexOffset(int32 MuscleIndex) { return MuscleVertexOffset[MuscleIndex]; };
		int32 GetMuscleVertexCount(int32 MuscleIndex) { return MuscleVertexCount[MuscleIndex]; };

		/* returns muscle activation of the specified muscle */
		FSolverReal GetMuscleActivation(int32 MuscleIndex) const { 
			if (MuscleActivation.IsValidIndex(MuscleIndex))
			{
				return MuscleActivation[MuscleIndex];
			}
			return FSolverReal(0);  
		};

		/* Sets muscle activation of the specified muscle and returns bool for success*/
		bool SetMuscleActivation(int32 MuscleIndex, FSolverReal InMuscleActivation)
		{
			if (MuscleActivation.IsValidIndex(MuscleIndex))
			{
				MuscleActivation[MuscleIndex] = FMath::Clamp(InMuscleActivation, FSolverReal(0), FSolverReal(1));
				return true;
			}
			return false;
		};

	private:	
		TArray<TArray<int32>> MuscleActivationElement;
		TArray<FIntVector2> OriginInsertionPair;
		TArray<FSolverReal> MuscleRestLength;
		TArray<FSolverReal> MuscleActivation;
		TArray<TArray<Chaos::PMatrix33d>> FiberDirectionMatrix;
		TArray<TArray<FSolverReal>> ContractionVolumeScale;
		TArray<FSolverReal> FiberLengthRatioAtMaxActivation;
		TArray<FSolverReal> MuscleLengthRatioThresholdForMaxActivation;
		TArray<FSolverReal> InflationVolumeScale;
		TArray<Chaos::FLinearCurve> LengthActivationCurves;
		TArray<int32> MuscleVertexOffset;
		TArray<int32> MuscleVertexCount;
	};

	template <typename T, typename ParticleType>
	using FMuscleActivationConstraints UE_DEPRECATED(5.8, "Deprecated. this class is to be deleted, use FMuscleActivationTetrahedralConstraints instead") = FMuscleActivationTetrahedralConstraints;

}// End namespace Chaos::Softs
