// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/Core.h"
#include "Chaos/Matrix.h"
#include "Chaos/Transform.h"
#include "Chaos/UniformGrid.h"
#include "Chaos/Vector.h"
#include "Chaos/Triangle.h"

//#include "ChaosFlesh/FleshCollectionUtility.h"
#include "Math/NumericLimits.h"
#include "Templates/IsIntegral.h"
#include "Chaos/Plane.h"
#include "Chaos/TriangleMeshImplicitObject.h"
#include "Chaos/Matrix.h"

namespace Chaos
{
	namespace Utilities
	{

		inline void GetTetFaces(
			const FIntVector4& Tet,
			FIntVector3& Face1,
			FIntVector3& Face2,
			FIntVector3& Face3,
			FIntVector3& Face4,
			const bool invert)
		{
			const int32 i = Tet[0];
			const int32 j = Tet[1];
			const int32 k = Tet[2];
			const int32 l = Tet[3];
			if (invert)
			{
				Face1 = { i, k, j };
				Face2 = { i, j, l };
				Face3 = { i, l, k };
				Face4 = { j, k, l };
			}
			else
			{
				Face1 = { i, j, k };
				Face2 = { i, l, j };
				Face3 = { i, k, l };
				Face4 = { j, l, k };
			}
		}

		inline int32
			GetMin(const FIntVector3& V)
		{
			return FMath::Min3(V[0], V[1], V[2]);
		}
		inline int32
			GetMid(const FIntVector3& V)
		{
			const int32 X = V[0]; const int32 Y = V[1]; const int32 Z = V[2];
			const int32 XmY = X - Y;
			const int32 YmZ = Y - Z;
			const int32 XmZ = X - Z;
			return (XmY * YmZ > -1 ? Y : XmY * XmZ < 1 ? X : Z);
		}
		inline int32
			GetMax(const FIntVector3& V)
		{
			return FMath::Max3(V[0], V[1], V[2]);
		}
		inline FIntVector3
			GetOrdered(const FIntVector3& V)
		{
			return FIntVector3(GetMin(V), GetMid(V), GetMax(V));
		}
		inline FIntVector4
			GetOrdered(const FIntVector4& V)
		{
			TArray<int32> VA = { V[0], V[1], V[2], V[3] };
			VA.Sort();
			return FIntVector4(VA[0], VA[1], VA[2], VA[3]);
		}

		inline void
			GetSurfaceElements(
				const TArray<FIntVector4>& Tets,
				TArray<FIntVector3>& SurfaceElements,
				const bool KeepInteriorFaces = false,
				const bool InvertFaces = false)
		{
			FIntVector3 Faces[4];

			if (KeepInteriorFaces)
			{
				SurfaceElements.Reserve(Tets.Num() * 4);
				for (const FIntVector4& Tet : Tets)
				{
					GetTetFaces(Tet, Faces[0], Faces[1], Faces[2], Faces[3], InvertFaces);
					for (int i = 0; i < 4; i++)
						SurfaceElements.Add(Faces[i]);
				}
			}
			else
			{
				typedef TMap<int32, TPair<uint8, FIntVector3>> ZToCount;
				typedef TMap<int32, ZToCount> YToZ;
				typedef TMap<int32, YToZ> XToY;
				XToY CoordToCount;

				int32 Idx = -1;
				int32 Count = 0;
				for (const FIntVector4& Tet : Tets)
				{
					++Idx;
					if (!(Tet[0] != Tet[1] &&
						Tet[0] != Tet[2] &&
						Tet[0] != Tet[3] &&
						Tet[1] != Tet[2] &&
						Tet[1] != Tet[3] &&
						Tet[2] != Tet[3]))
					{
						//UE_LOGF(LogChaosFlesh, Display, "Skipping degenerate tet %d of %d.", Idx, Tets.Num());
						continue;
					}

					GetTetFaces(Tet, Faces[0], Faces[1], Faces[2], Faces[3], InvertFaces);
					for (int i = 0; i < 4; i++)
					{
						const FIntVector3 OFace = GetOrdered(Faces[i]);
						check(OFace[0] <= OFace[1] && OFace[1] <= OFace[2]);
						const int32 OA = OFace[0];
						const int32 OB = OFace[1];
						const int32 OC = OFace[2];

						auto& zc = CoordToCount.FindOrAdd(OFace[0]).FindOrAdd(OFace[1]);
						auto zcIt = zc.Find(OFace[2]);
						if (zcIt == nullptr)
						{
							zc.Add(OFace[2], TPair<uint8, FIntVector3>((uint8)1, Faces[i]));
							// Increment our count of lone faces.
							Count++;
						}
						else
						{
							zcIt->Key++;
							// Since we're talking about shared faces, the only way we'd
							// get a face instanced more than twice is if we had a degenerate 
							// tet mesh.
							Count--;
						}
					}
				}

				size_t nonManifold = 0;
				SurfaceElements.Reserve(Count);
				for (auto& xyIt : CoordToCount)
				{
					const YToZ& yz = xyIt.Value;
					for (auto& yzIt : yz)
					{
						const ZToCount& zc = yzIt.Value;
						for (auto& zcIt : zc)
						{
							const uint8 FaceCount = zcIt.Value.Key;
							if (FaceCount == 1)
							{
								SurfaceElements.Add(zcIt.Value.Value);
							}
							else if (FaceCount > 2)
							{
								//UE_LOGF(LogChaosFlesh, Display, "WARNING: Non-manifold tetrahedral mesh detected (face [%d, %d, %d] use count %d).", zcIt->second.second[0], zcIt->second.second[1], zcIt->second.second[2], FaceCount);
								nonManifold++;
							}
						}
					}
				}
				//if (nonManifold)
				//	UE_LOGF(LogChaosFlesh, Display, "WARNING: Encountered %lld non-manifold tetrahedral mesh faces.", nonManifold);
			}
		}
	

		inline void FindClosestTriangle(
			const TArray<TVector<FRealSingle, 3>>& Positions,
			const TArray<FIntVector3>& SurfaceElements,
			const TArray<TVector<FRealSingle, 3>>& TrianglePositions,
			TArray<int32>& SurfaceTriangleIndices,
			TArray<TVector<FRealSingle, 3>>& BarycentricWeights,
			const FRealSingle BoxSize = FRealSingle(10.))
		{
			SurfaceTriangleIndices.Init(-1, Positions.Num());
			BarycentricWeights.Init(TVector<FRealSingle, 3>(0.), Positions.Num());
			TArray<TVector<FRealSingle, 3>> TrianglePositionsTemp = TrianglePositions;
			TParticles<FRealSingle, 3> TriangleParticles(MoveTemp(TrianglePositionsTemp));
			TArray<FIntVector3> SurfaceElementsTemp = SurfaceElements;
			TArray<uint16> MaterialIndices;
			//auto TriangleMesh = MakeUnique<Chaos::FTriangleMeshImplicitObject>(MoveTemp(TriangleParticles), MoveTemp(SurfaceElementsTemp), MoveTemp(MaterialIndices));
			//Chaos::FTriangleMeshImplicitObject TriangleMeshImplicit(MoveTemp(TriangleParticles), MoveTemp(SurfaceElementsTemp), MoveTemp(TArray<uint16>()));
			TArray<TVec3<int32>> Elements;
			Elements.Init(TVec3<int32>(0), SurfaceElements.Num());
			for (int32 i = 0; i < SurfaceElements.Num(); i++)
			{
				Elements[i] = TVec3<int32>(SurfaceElements[i][0], SurfaceElements[i][1], SurfaceElements[i][2]);
			}
			auto TriangleMesh = MakeUnique<FTriangleMeshImplicitObject>(MoveTemp(TriangleParticles), MoveTemp(Elements), MoveTemp(MaterialIndices));

			for (int32 p = 0; p < Positions.Num(); p++)
			{
				TVector<FReal, 3> MinCorner = { Positions[p][0] - BoxSize / (FReal)2., Positions[p][1] - BoxSize / (FReal)2. , Positions[p][2] - BoxSize / (FReal)2. };
				TVector<FReal, 3> MaxCorner = { Positions[p][0] + BoxSize / (FReal)2., Positions[p][1] + BoxSize / (FReal)2. , Positions[p][2] + BoxSize / (FReal)2. };

				Chaos::FAABB3 QueryBox(MinCorner, MaxCorner);

				int32 MinTriangleIndex = -1;
				FRealSingle ClosestDistance = (FRealSingle)1e6;
				TVector<FRealSingle, 3> ClosestBary;
				TriangleMesh->VisitTriangles(QueryBox, Chaos::FRigidTransform3(FMatrix44d::Identity), [&](const FTriangle& Triangle, const int32 TriangleIndex, const int32 VertexIndex0, const int32 VertexIndex1, const int32 VertexIndex2)
					{
						TVector<FRealSingle, 3> Bary;
						TVector<FRealSingle, 3> ClosestPoint = Chaos::FindClosestPointAndBaryOnTriangle(TrianglePositions[VertexIndex0], TrianglePositions[VertexIndex1], TrianglePositions[VertexIndex2], Positions[p], Bary);
						FRealSingle CurrentDistance = (Positions[p] - ClosestPoint).Size();
						if (CurrentDistance < ClosestDistance)
						{
							ClosestDistance = CurrentDistance;
							MinTriangleIndex = TriangleIndex;
							ClosestBary = Bary;
						}

					});
				BarycentricWeights[p] = ClosestBary;
				SurfaceTriangleIndices[p] = MinTriangleIndex;
			}
		}

	}
}
