// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Chaos/Core.h"
#include "Chaos/Vector.h"
#include "Chaos/Matrix.h"
#include "Chaos/Utilities.h"

namespace Chaos
{
	/**
	 * A block-diagonal matrix specifically for use with Mass/Inertia (or more usually inverse mass and inertia)
	 * and TDenseMatrix math used by the constraint solvers.
	 */
	class FMassMatrix
	{
	public:
		FReal M() const
		{
			return Mass;
		}
		
		const FReal I(int32 RowIndex, int32 ColIndex) const
		{
			checkSlow(RowIndex < 3);
			checkSlow(ColIndex < 3);
			return Inertia.M[RowIndex][ColIndex];
		}

		static FMassMatrix Make(const FReal InM, const FMatrix33& InI)
		{
			return FMassMatrix(InM, InI);
		}

		static FMassMatrix Make(const FReal InM, FMatrix33&& InI)
		{
			return FMassMatrix(InM, MoveTemp(InI));
		}

		static FMassMatrix Make(const FReal InM, const FRotation3& Q, const FMatrix33& InI)
		{
			return FMassMatrix(InM, Q, InI);
		}

	private:
		FMassMatrix(const FReal InM, const FMatrix33& InI)
			: Mass(InM)
			, Inertia(InI)
		{
		}

		FMassMatrix(const FReal InM, FMatrix33&& InI)
			: Mass(InM)
			, Inertia(MoveTemp(InI))
		{
		}

		FMassMatrix(const FReal InM, const FRotation3& Q, const FMatrix33& InI)
			: Mass(InM)
			, Inertia(Utilities::ComputeWorldSpaceInertia(Q, InI))
		{
		}

		FReal Mass;
		FMatrix33 Inertia;
	};

	/**
	 * A matrix with run-time variable dimensions, up to an element limit defined at compile-time.
	 *
	 * Elements are stored in row-major order (i.e., elements in a row are adjacent in memory). Note
	 * that FMatrix stores elements in column-major order so that we can access the columns quickly
	 * which is handy when you have rotation matrices and want the spatial axes. We don't care about
	 * that so we use the more conventional row-major indexing and matching storage.
	 */
	template<int32 T_MAXELEMENTS>
	class TDenseMatrix
	{
	public:
		static const int32 MaxElements = T_MAXELEMENTS;

		TDenseMatrix()
			: NRows(0)
			, NCols(0)
		{
		}

		TDenseMatrix(const int32 InNRows, const int32 InNCols)
			: NRows(InNRows)
			, NCols(InNCols)
		{
		}

		TDenseMatrix(const TDenseMatrix<MaxElements>& A)
		{
			*this = A;
		}

		TDenseMatrix<MaxElements>& operator=(const TDenseMatrix<MaxElements>& A)
		{
			SetDimensions(A.NumRows(), A.NumColumns());
			for (int32 Index = 0; Index < NumElements(); ++Index)
			{
				M[Index] = A.M[Index];
			}
			return *this;
		}

		/**
		 * The number of rows in the matrix.
		 */
		FORCEINLINE int32 NumRows() const
		{
			return NRows;
		}

		/**
		 * The number of columns in the matrix.
		 */
		FORCEINLINE int32 NumColumns() const
		{
			return NCols;
		}

		/**
		 * The number of elements in the matrix.
		 */
		FORCEINLINE int32 NumElements() const
		{
			return NRows * NCols;
		}

		/**
		 * Set the dimensions of the matrix, but do not initialize any values.
		 * This will invalidate any existing data.
		 */
		FORCEINLINE void SetDimensions(const int32 InNumRows, const int32 InNumColumns)
		{
			check(InNumRows * InNumColumns <= MaxElements);
			NRows = InNumRows;
			NCols = InNumColumns;
		}

		/**
		 * Add uninitialized rows to the matrix. This will not invalidate data in any previously added rows,
		 * so it can be used to build NxM matrices where M is known, but N is calculated later.
		 * /return the index of the first new row.
		 */
		FORCEINLINE int32 AddRows(const int32 InNumRows)
		{
			check((NumRows() + InNumRows) * NumColumns() <= MaxElements);
			const int32 NewRowIndex = NRows;
			NRows = NRows + InNumRows;
			return NewRowIndex;
		}

		/**
		 * Return a writable reference to the element at the specified row and column.
		 */
		FORCEINLINE FReal& At(const int32 RowIndex, const int32 ColumnIndex)
		{
			checkSlow(RowIndex < NumRows());
			checkSlow(ColumnIndex < NumColumns());
			return M[ElementIndex(RowIndex, ColumnIndex)];
		}

		/**
		 * Return a read-only reference to the element at the specified row and column.
		 */
		FORCEINLINE const FReal& At(const int32 RowIndex, const int32 ColumnIndex) const
		{
			checkSlow(RowIndex < NumRows());
			checkSlow(ColumnIndex < NumColumns());
			return M[ElementIndex(RowIndex, ColumnIndex)];
		}

		/**
		 * Set the dimensions and initial values of the matrix.
		 */
		void Init(const int32 InNRows, const int32 InNCols, FReal V)
		{
			SetDimensions(InNRows, InNCols);
			Set(V);
		}

		/**
		 * Set the element
		 */
		FORCEINLINE void SetAt(const int32 RowIndex, const int32 ColumnIndex, const FReal V)
		{
			At(RowIndex, ColumnIndex) = V;
		}

		/**
		 * Set all elements to 'V'.
		 */
		void Set(FReal V)
		{
			for (int32 II = 0; II < NumElements(); ++II)
			{
				M[II] = V;
			}
		}

		/**
		 * Set the diagonal elements to 'V'. Does not set off-diagonal elements.
		 * /see MakeDiagonal
		 */
		void SetDiagonal(FReal V)
		{
			check(NumRows() == NumColumns());
			for (int32 II = 0; II < NRows; ++II)
			{
				At(II, II) = V;
			}
		}

		/**
		 * Set the "Num" diagonal elements starting from ("Start", "Start") to "V". Does not set off-diagonal elements.
		 */
		void SetDiagonalAt(int32 Start, int32 Num, FReal V)
		{
			check(Start + Num <= NumRows());
			check(Start + Num <= NumColumns());
			for (int32 II = 0; II < Num; ++II)
			{
				int32 JJ = Start + II;
				At(JJ, JJ) = V;
			}
		}

		/**
		 * Starting from element ("RowIndex", "ColumnIndex"), set the next "NumV" elements in the row to the values in "V".
		 */
		void SetRowAt(const int32 RowIndex, const int32 ColumnIndex, const FReal* V, const int32 NumV)
		{
			check(RowIndex + NumV < NumRows());
			check(ColumnIndex < NumColumns());
			FReal* Row = &At(RowIndex, ColumnIndex);
			for (int32 II = 0; II < NumV; ++II)
			{
				*Row++ = *V++;
			}
		}

		void SetRowAt(const int32 RowIndex, const int32 ColumnIndex, const FVec3& V)
		{
			SetRowAt(RowIndex, ColumnIndex, V[0], V[1], V[2]);
		}

		void SetRowAt(const int32 RowIndex, const int32 ColumnIndex, const FReal V0, const FReal V1, const FReal V2)
		{
			check(RowIndex + 1 <= NumRows());
			check(ColumnIndex + 3 <= NumColumns());
			FReal* Row = &At(RowIndex, ColumnIndex);
			*Row++ = V0;
			*Row++ = V1;
			*Row++ = V2;
		}

		/**
		 * Starting from element ("RowIndex", "ColumnIndex"), set the next "NumV" elements in the column to the values in "V".
		 */
		void SetColumnAt(const int32 RowIndex, const int32 ColumnIndex, const FReal* V, const int32 NumV)
		{
			check(RowIndex + NumV <= NumRows());
			check(ColumnIndex + 1 <= NumColumns());
			for (int32 II = 0; II < NumV; ++II)
			{
				At(RowIndex + II, ColumnIndex) = V[II];
			}
		}

		void SetColumnAt(const int32 RowIndex, const int32 ColumnIndex, const FVec3& V)
		{
			check(RowIndex + 3 <= NumRows());
			check(ColumnIndex + 1 <= NumColumns());
			At(RowIndex + 0, ColumnIndex) = V[0];
			At(RowIndex + 1, ColumnIndex) = V[1];
			At(RowIndex + 2, ColumnIndex) = V[2];
		}

		/**
		 * Set the block starting at ("RowOffset", "ColumnOffset") from the specified matrix "V"
		 */
		template<int32 T_EA>
		void SetBlockAt(const int32 RowOffset, const int32 ColumnOffset, const TDenseMatrix<T_EA>& V)
		{
			check(RowOffset + V.NumRows() <= NumRows());
			check(ColumnOffset + V.NumColumns() <= NumColumns());
			for (int32 II = 0; II < V.NumRows(); ++II)
			{
				for (int32 JJ = 0; JJ < V.NumColumns(); ++JJ)
				{
					At(II + RowOffset, JJ + ColumnOffset) = V.At(II, JJ);
				}
			}
		}

		/**
		 * Set the 3x3 block starting at ("RowOffset", "ColumnOffset") from the specified 3x3 matrix "V" (note: assumes the input UE matrix is column-major order)
		 */
		void SetBlockAt(const int32 RowOffset, const int32 ColumnOffset, const FMatrix33& V)
		{
			check(RowOffset + 3 <= NumRows());
			check(ColumnOffset + 3 <= NumColumns());
			for (int32 II = 0; II < 3; ++II)
			{
				for (int32 JJ = 0; JJ < 3; ++JJ)
				{
					At(II + RowOffset, JJ + ColumnOffset) = V.M[JJ][II];
				}
			}
		}

		/**
		 * Set the specified 3x3 block to a diagonal matrix with the specified diagonal and off-diagonal values.
		 */
		void SetBlockAtDiagonal33(const int32 RowOffset, const int32 ColumnOffset, const FReal VDiag, const FReal VOffDiag)
		{
			check(RowOffset + 3 <= NumRows());
			check(ColumnOffset + 3 <= NumColumns());
			FReal* Row0 = &At(RowOffset + 0, ColumnOffset);
			*Row0++ = VDiag;
			*Row0++ = VOffDiag;
			*Row0++ = VOffDiag;
			FReal* Row1 = &At(RowOffset + 1, ColumnOffset);
			*Row1++ = VOffDiag;
			*Row1++ = VDiag;
			*Row1++ = VOffDiag;
			FReal* Row2 = &At(RowOffset + 2, ColumnOffset);
			*Row2++ = VOffDiag;
			*Row2++ = VOffDiag;
			*Row2++ = VDiag;
		}

		//
		// Factory methods
		//

		/**
		 * Create a matrix with the specified dimensions, but all elements are uninitialized.
		 */
		static TDenseMatrix<MaxElements> Make(const int32 InNumRows, const int32 InNumCols)
		{
			return TDenseMatrix<MaxElements>(InNumRows, InNumCols);
		}

		/**
		 * Create a matrix with the specified dimensions, and initialize all elements with "V".
		 */
		static TDenseMatrix<MaxElements> Make(const int32 InNumRows, const int32 InNumCols, const FReal V)
		{
			return TDenseMatrix<MaxElements>(InNumRows, InNumCols, V);
		}

		/**
		 * Create a matrix with the specified elements supplied as an array in row-major order 
		 * (i.e., the first N elements are for Row 0, the next N for Row 1, etc., where N is the number of columns).
		 */
		static TDenseMatrix<MaxElements> Make(const int32 InNumRows, const int32 InNumCols, const FReal* V, const int32 VLen)
		{
			return TDenseMatrix<MaxElements>(InNumRows, InNumCols, V, VLen);
		}

		/**
		 * Create a copy of the 3x1 columns vector.
		 */
		static TDenseMatrix<MaxElements> Make(const FVec3& InM)
		{
			TDenseMatrix<MaxElements> M(3, 1);
			M.At(0, 0) = InM[0];
			M.At(1, 0) = InM[1];
			M.At(2, 0) = InM[2];
			return M;
		}

		/**
		 * Create a copy of the 3x3 matrix.
		 */
		static TDenseMatrix<MaxElements> Make(const FMatrix33& InM)
		{
			// NOTE: UE matrices are Column-major (columns are sequential in memory), but DenseMatrix is Row-major (rows are sequential in memory)
			TDenseMatrix<MaxElements> M(3, 3);
			M.At(0, 0) = InM.M[0][0];
			M.At(0, 1) = InM.M[1][0];
			M.At(0, 2) = InM.M[2][0];
			M.At(1, 0) = InM.M[0][1];
			M.At(1, 1) = InM.M[1][1];
			M.At(1, 2) = InM.M[2][1];
			M.At(2, 0) = InM.M[0][2];
			M.At(2, 1) = InM.M[1][2];
			M.At(2, 2) = InM.M[2][2];
			return M;
		}

		/**
		 * Create a matrix with all elemets set to zero, except the diagonal elements.
		 */
		static TDenseMatrix<MaxElements> MakeDiagonal(const int32 InNumRows, const int32 InNumCols, const FReal D)
		{
			TDenseMatrix<MaxElements> M(InNumRows, InNumCols);
			for (int32 I = 0; I < InNumRows; ++I)
			{
				for (int32 J = 0; J < InNumCols; ++J)
				{
					M.At(I, J) = (I == J) ? D : 0;
				}
			}
			return M;
		}

		/**
		 * Create an identity matrix.
		 */
		static TDenseMatrix<MaxElements> MakeIdentity(const int32 InDim)
		{
			return MakeDiagonal(InDim, InDim, (FReal)1);
		}

		//
		// Math operations
		//

		/**
		 * Return the transpose of 'A'.
		 */
		template<int32 T_EA>
		static TDenseMatrix<MaxElements> Transpose(const TDenseMatrix<T_EA>& A)
		{
			TDenseMatrix<T_MAXELEMENTS> Result(A.NumColumns(), A.NumRows());
			for (int32 IRow = 0; IRow < Result.NumRows(); ++IRow)
			{
				for (int32 ICol = 0; ICol < Result.NumColumns(); ++ICol)
				{
					Result.At(IRow, ICol) = A.At(ICol, IRow);
				}
			}
			return Result;
		}

		/**
		 * Copy a matrix and set each element to its negative.
		 */
		template<int32 T_EA>
		static TDenseMatrix<MaxElements> Negate(const TDenseMatrix<T_EA>& A)
		{
			// @todo(ccaulfield): optimize
			TDenseMatrix<T_MAXELEMENTS> Result(A.NumRows(), A.NumColumns());
			for (int32 IRow = 0; IRow < Result.NumRows(); ++IRow)
			{
				for (int32 ICol = 0; ICol < Result.NumColumns(); ++ICol)
				{
					Result.At(IRow, ICol) = -A.At(IRow, ICol);
				}
			}
			return Result;
		}

		/**
		 * Return C = A + B
		 */
		template<int32 T_EA, int32 T_EB>
		static TDenseMatrix<MaxElements> Add(const TDenseMatrix<T_EA>& A, const TDenseMatrix<T_EB>& B)
		{
			// @todo(ccaulfield): optimize
			check(A.NumColumns() == B.NumColumns());
			check(A.NumRows() == B.NumRows());
			TDenseMatrix<T_MAXELEMENTS> Result(A.NumRows(), A.NumColumns());
			for (int32 IRow = 0; IRow < Result.NumRows(); ++IRow)
			{
				for (int32 ICol = 0; ICol < Result.NumColumns(); ++ICol)
				{
					Result.At(IRow, ICol) = A.At(IRow, ICol) + B.At(IRow, ICol);
				}
			}
			return Result;
		}

		/**
		 * Return C = A + B, where A and B are symetric.
		 */
		template<int32 T_EA, int32 T_EB>
		static TDenseMatrix<MaxElements> Add_Symmetric(const TDenseMatrix<T_EA>& A, const TDenseMatrix<T_EB>& B)
		{
			// @todo(ccaulfield): optimize
			check(A.NumColumns() == B.NumColumns());
			check(A.NumRows() == B.NumRows());
			TDenseMatrix<T_MAXELEMENTS> Result(A.NumRows(), A.NumColumns());
			for (int32 IRow = 0; IRow < Result.NumRows(); ++IRow)
			{
				for (int32 ICol = IRow; ICol < Result.NumColumns(); ++ICol)
				{
					FReal V = A.At(IRow, ICol) + B.At(IRow, ICol);
					Result.At(IRow, ICol) = V;
					Result.At(ICol, IRow) = V;
				}
			}
			return Result;
		}

		/**
		 * Return C = A - B
		 */
		template<int32 T_EA, int32 T_EB>
		static TDenseMatrix<MaxElements> Subtract(const TDenseMatrix<T_EA>& A, const TDenseMatrix<T_EB>& B)
		{
			// @todo(ccaulfield): optimize
			check(A.NumColumns() == B.NumColumns());
			check(A.NumRows() == B.NumRows());
			TDenseMatrix<T_MAXELEMENTS> Result(A.NumRows(), A.NumColumns());
			for (int32 IRow = 0; IRow < Result.NumRows(); ++IRow)
			{
				for (int32 ICol = 0; ICol < Result.NumColumns(); ++ICol)
				{
					Result.At(IRow, ICol) = A.At(IRow, ICol) - B.At(IRow, ICol);
				}
			}
			return Result;
		}

		/**
		 * Return C = A x B, the product of A and B where each element of C is Cij = DotProduct(A.GetRow(i), B.GetColumns(j)).
		 * /see MultiplyAtB, MultiplyABt, MultipltAtBt.
		 */
		template<int32 T_EA, int32 T_EB>
		static TDenseMatrix<MaxElements> MultiplyAB(const TDenseMatrix<T_EA>& A, const TDenseMatrix<T_EB>& B)
		{
			// @todo(ccaulfield): optimize
			check(A.NumColumns() == B.NumRows());
			TDenseMatrix<T_MAXELEMENTS> Result(A.NumRows(), B.NumColumns());
			for (int32 IRow = 0; IRow < Result.NumRows(); ++IRow)
			{
				for (int32 ICol = 0; ICol < Result.NumColumns(); ++ICol)
				{
					FReal V = 0;
					for (int32 II = 0; II < A.NumColumns(); ++II)
					{
						V += A.At(IRow, II) * B.At(II, ICol);
					}
					Result.At(IRow, ICol) = V;
				}
			}
			return Result;
		}

		/**
		 * Return C = Transpose(A) x B.
		 * /see MultiplyAB, MultiplyABt, MultipltAtBt.
		 */
		template<int32 T_EA, int32 T_EB>
		static TDenseMatrix<MaxElements> MultiplyAtB(const TDenseMatrix<T_EA>& A, const TDenseMatrix<T_EB>& B)
		{
			// @todo(ccaulfield): optimize
			check(A.NumRows() == B.NumRows());
			TDenseMatrix<T_MAXELEMENTS> Result(A.NumColumns(), B.NumColumns());
			for (int32 IRow = 0; IRow < Result.NumRows(); ++IRow)
			{
				for (int32 ICol = 0; ICol < Result.NumColumns(); ++ICol)
				{
					FReal V = 0;
					for (int32 II = 0; II < A.NumRows(); ++II)
					{
						V += A.At(II, IRow) * B.At(II, ICol);
					}
					Result.At(IRow, ICol) = V;
				}
			}
			return Result;
		}

		/**
		 * Return C = A x Transpose(B).
		 * /see MultiplyAB, MultiplyAtB, MultipltAtBt.
		 */
		template<int32 T_EA, int32 T_EB>
		static TDenseMatrix<MaxElements> MultiplyABt(const TDenseMatrix<T_EA>& A, const TDenseMatrix<T_EB>& B)
		{
			// @todo(ccaulfield): optimize
			check(A.NumColumns() == B.NumColumns());
			TDenseMatrix<T_MAXELEMENTS> Result(A.NumRows(), B.NumRows());
			for (int32 IRow = 0; IRow < Result.NumRows(); ++IRow)
			{
				for (int32 ICol = 0; ICol < Result.NumColumns(); ++ICol)
				{
					FReal V = 0;
					for (int32 II = 0; II < A.NumColumns(); ++II)
					{
						V += A.At(IRow, II) * B.At(ICol, II);
					}
					Result.At(IRow, ICol) = V;
				}
			}
			return Result;
		}

		/**
		 * Return C = Transpose(A) x Transpose(B).
		 * /see MultiplyAB, MultiplyAtB, MultipltABt.
		 */
		template<int32 T_EA, int32 T_EB>
		static TDenseMatrix<MaxElements> MultiplyAtBt(const TDenseMatrix<T_EA>& A, const TDenseMatrix<T_EB>& B)
		{
			// @todo(ccaulfield): optimize
			check(A.NumRows() == B.NumColumns());
			TDenseMatrix<T_MAXELEMENTS> Result(A.NumColumns(), B.NumRows());
			for (int32 IRow = 0; IRow < Result.NumRows(); ++IRow)
			{
				for (int32 ICol = 0; ICol < Result.NumColumns(); ++ICol)
				{
					FReal V = 0;
					for (int32 II = 0; II < A.NumRows(); ++II)
					{
						V += A.At(II, IRow) * B.At(ICol, II);
					}
					Result.At(IRow, ICol) = V;
				}
			}
			return Result;
		}

		/**
		 * Return C = A x B, where the results is known to be symmetric.
		 * /see MultiplyAtB.
		 */
		template<int32 T_EA, int32 T_EB>
		static TDenseMatrix<MaxElements> MultiplyAB_Symmetric(const TDenseMatrix<T_EA>& A, const TDenseMatrix<T_EB>& B)
		{
			// @todo(ccaulfield): optimize
			check(A.NumColumns() == B.NumRows());
			TDenseMatrix<T_MAXELEMENTS> Result(A.NumRows(), B.NumColumns());
			for (int32 IRow = 0; IRow < Result.NumRows(); ++IRow)
			{
				for (int32 ICol = IRow; ICol < Result.NumColumns(); ++ICol)
				{
					FReal V = 0;
					for (int32 II = 0; II < A.NumColumns(); ++II)
					{
						V += A.At(IRow, II) * B.At(II, ICol);
					}
					Result.At(IRow, ICol) = V;
					Result.At(ICol, IRow) = V;
				}
			}
			return Result;
		}

		/**
		 * Return C = A + B x C, where the A and (B x C) are known to be symmetric.
		 * /see MultiplyAtB, Add_Symmetric.
		 */
		template<int32 T_EA, int32 T_EB, int32 T_EC>
		static TDenseMatrix<MaxElements> MultiplyBCAddA_Symmetric(const TDenseMatrix<T_EA>& A, const TDenseMatrix<T_EB>& B, const TDenseMatrix<T_EC>& C)
		{
			// @todo(ccaulfield): optimize
			check(B.NumColumns() == C.NumRows());
			check(A.NumRows() == B.NumRows());
			check(A.NumRows() == C.NumColumns());
			TDenseMatrix<T_MAXELEMENTS> Result(A.NumRows(), A.NumColumns());
			for (int32 IRow = 0; IRow < Result.NumRows(); ++IRow)
			{
				for (int32 ICol = IRow; ICol < Result.NumColumns(); ++ICol)
				{
					FReal V = 0;
					for (int32 II = 0; II < B.NumColumns(); ++II)
					{
						V += B.At(IRow, II) * C.At(II, ICol);
					}
					FReal VA = A.At(IRow, ICol);
					Result.At(IRow, ICol) = VA + V;
					Result.At(ICol, IRow) = VA + V;
				}
			}
			return Result;
		}

		/**
		 * Return C = A x B, where A is an Nx6 matrix, and B is a 6x6 mass matrix (Mass in upper left 3x3 diagonals, Inertia in lower right 3x3).
		 * C = |A0 A1| * |M 0| = |A0.M A1.I|
		 *               |0 I|
		 */
		template<int32 T_EA>
		static TDenseMatrix<MaxElements> MultiplyAB(const TDenseMatrix<T_EA>& A, const FMassMatrix& B)
		{
			check(A.NumColumns() == 6);

			TDenseMatrix<T_MAXELEMENTS> Result = TDenseMatrix<T_MAXELEMENTS>::Make(A.NumRows(), 6);

			// Calculate columns 0-2
			for (int32 IRow = 0; IRow < Result.NumRows(); ++IRow)
			{
				for (int32 ICol = 0; ICol < 3; ++ICol)
				{
					Result.At(IRow, ICol) = A.At(IRow, ICol) * B.M();
				}
			}

			// Calculate columns 3-5
			for (int32 IRow = 0; IRow < Result.NumRows(); ++IRow)
			{
				for (int32 ICol = 3; ICol < 6; ++ICol)
				{
					FReal V = 0;
					for (int32 KK = 3; KK < 6; ++KK)
					{
						V += A.At(IRow, KK) * B.I(KK - 3, ICol - 3);
					}
					Result.At(IRow, ICol) = V;
				}
			}

			return Result;
		}


		/**
		 * Return C = A x B, where B is an 6xN matrix, and A is a 6x6 mass matrix (Mass in upper left 3x3 diagonals, Inertia in lower right 3x3).
		 * C = |M 0| * |B0| = |M.B0|
		 *     |0 I|   |B1|   |I.B1|
		 */
		template<int32 T_EB>
		static TDenseMatrix<MaxElements> MultiplyAB(const FMassMatrix& A, const TDenseMatrix<T_EB>& B)
		{
			check(B.NumRows() == 6);

			TDenseMatrix<T_MAXELEMENTS> Result = TDenseMatrix<T_MAXELEMENTS>::Make(6, B.NumColumns());

			// Calculate rows 0-2
			for (int32 IRow = 0; IRow < 3; ++IRow)
			{
				for (int32 ICol = 0; ICol < B.NumColumns(); ++ICol)
				{
					Result.At(IRow, ICol) = A.M() * B.At(IRow, ICol);
				}
			}

			// Calculate rows 3-5
			for (int32 IRow = 3; IRow < 6; ++IRow)
			{
				for (int32 ICol = 0; ICol < B.NumColumns(); ++ICol)
				{
					FReal V = 0;
					for (int32 KK = 3; KK < 6; ++KK)
					{
						V += A.I(IRow - 3, KK - 3) * B.At(KK, ICol);
					}
					Result.At(IRow, ICol) = V;
				}
			}

			return Result;
		}

		/**
		 * Return C = A x Transpose(B), where B is an Nx6 matrix, and A is a 6x6 mass matrix (Mass in upper left 3x3 diagonals, Inertia in lower right 3x3).
		 * C = |M 0| * |B0 B1|(T) = |M.B0(T)|
		 *     |0 I|                |I.B1(T)|
		 */
		template<int32 T_EB>
		static TDenseMatrix<MaxElements> MultiplyABt(const FMassMatrix& A, const TDenseMatrix<T_EB>& B)
		{
			check(B.NumColumns() == 6);

			TDenseMatrix<T_MAXELEMENTS> Result = TDenseMatrix<T_MAXELEMENTS>::Make(6, B.NumRows());

			// Calculate rows 0-2
			for (int32 IRow = 0; IRow < 3; ++IRow)
			{
				for (int32 ICol = 0; ICol < B.NumRows(); ++ICol)
				{
					Result.At(IRow, ICol) = A.M() * B.At(ICol, IRow);
				}
			}

			// Calculate rows 3-5
			for (int32 IRow = 3; IRow < 6; ++IRow)
			{
				for (int32 ICol = 0; ICol < B.NumRows(); ++ICol)
				{
					FReal V = 0;
					for (int32 KK = 3; KK < 6; ++KK)
					{
						V += A.I(IRow - 3, KK - 3) * B.At(ICol, KK);
					}
					Result.At(IRow, ICol) = V;
				}
			}

			return Result;
		}


		/**
		 * Return C = A x V, where A is an MxN matrix, and V a real number.
		 */
		template<int32 T_EA>
		static TDenseMatrix<MaxElements> Multiply(const TDenseMatrix<T_EA>& A, const FReal V)
		{
			// @todo(ccaulfield): optimize
			TDenseMatrix<T_MAXELEMENTS> Result(A.NumRows(), A.NumColumns());
			for (int32 IRow = 0; IRow < Result.NumRows(); ++IRow)
			{
				for (int32 ICol = 0; ICol < Result.NumColumns(); ++ICol)
				{
					Result.At(IRow, ICol) = A.At(IRow, ICol) * V;
				}
			}
			return Result;
		}

		/**
		 * Return C = A x V, where A is an MxN matrix, and V a real number.
		 */
		template<int32 T_EA>
		static TDenseMatrix<MaxElements> Multiply(const FReal V, const TDenseMatrix<T_EA>& A)
		{
			return Multiply(A, V);
		}

		/**
		 * Return C = A / V, where A is an MxN matrix, and V a real number.
		 */
		template<int32 T_EA>
		static TDenseMatrix<MaxElements> Divide(const TDenseMatrix<T_EA>& A, const FReal V)
		{
			// @todo(ccaulfield): optimize
			TDenseMatrix<T_MAXELEMENTS> Result(A.NumRows(), A.NumColumns());
			for (int32 IRow = 0; IRow < Result.NumRows(); ++IRow)
			{
				for (int32 ICol = 0; ICol < Result.NumColumns(); ++ICol)
				{
					Result.At(IRow, ICol) = A.At(IRow, ICol) / V;
				}
			}
			return Result;
		}

		/**
		 * Return C = At x B. If A and B are column vectors (Nx1 matrices), this is a vector dot product.
		 */
		template<int32 T_EA, int32 T_EB>
		static TDenseMatrix<MaxElements> DotProduct(const TDenseMatrix<T_EA>& A, const TDenseMatrix<T_EB>& B)
		{
			return MultiplyAtB(A, B);
		}

	private:
		TDenseMatrix(const int32 InNRows, const int32 InNCols, const FReal V)
			: NRows(InNRows)
			, NCols(InNCols)
		{
			for (int32 I = 0; I < NumElements(); ++I)
			{
				M[I] = V;
			}
		}

		TDenseMatrix(const int32 InNRows, const int32 InNCols, const FReal* V, const int32 N)
			: NRows(InNRows)
			, NCols(InNCols)
		{
			int32 NLimited = FMath::Min<int32>(NumElements(), N);
			for (int32 I = 0; I < NLimited; ++I)
			{
				M[I] = V[I];
			}
		}

		FORCEINLINE int32 ElementIndex(const int32 RowIndex, const int32 ColumnIndex) const
		{
			return RowIndex * NCols + ColumnIndex;
		}

		FReal M[MaxElements];
		int32 NRows;
		int32 NCols;
	};

	/**
	 * Methods to solves sets of Linear equations stored as
	 *	AX = B
	 * where A is an NxN matrix, and X.B are Nx1 column vectors.
	 */
	class FDenseMatrixSolver
	{
	public:

		/**
		 * Overwrite A with its Cholesky Factor (A must be Positive Definite).
		 * See "Matrix Computations, 4th Edition" Section 4.2, Golub & Van Loan.
		 *
		 * The Cholesky Factor of A is G (Gt its transpose), where A = GGt. G is lower triangular.
		 */
		template<int32 T_E>
		static bool CholeskyFactorize(TDenseMatrix<T_E>& A)
		{
			check(A.NumRows() == A.NumColumns());
			const int32 N = A.NumRows();
			for (int32 I = 0; I < N; ++I)
			{
				for (int32 J = I; J < N; ++J)
				{
					FReal Sum = A.At(I, J);
					for (int32 K = I - 1; K >= 0; --K)
					{
						Sum -= A.At(I, K) * A.At(J, K);
					}
					if (I == J)
					{
						if (Sum <= 0)
						{
							// Not positive definite (rounding?)
							return false;
						}
						A.At(I, J) = FMath::Sqrt(Sum);
					}
					else
					{
						A.At(J, I) = Sum / A.At(I, I);
					}
				}
			}

			for (int32 I = 0; I < N; ++I)
			{
				for (int32 J = 0; J < I; ++J)
				{
					A.At(J, I) = 0;
				}
			}

			return true;
		}

		/**
		 * This solves AX = B, where A is positive definite and has been Cholesky Factorized to produce G, 
		 * where A = GGt, G is lower triangular.
		 *
		 * This is a helper method for SolvePositiveDefinite, or useful if you need to reuse the 
		 * Cholesky Factor and therefore calculated it yourself.
		 *
		 * \see SolvePositiveDefinite
		 */
		template<int32 T_EA, int32 T_EB, int32 T_EX>
		static void SolveCholeskyFactorized(const TDenseMatrix<T_EA>& G, const TDenseMatrix<T_EB>& B, TDenseMatrix<T_EX>& X)
		{
			check(B.NumColumns() == 1);
			check(G.NumRows() == B.NumRows());

			const int32 N = G.NumRows();
			X.SetDimensions(N, 1);

			// By definition: 
			//		A.X = G.Gt.X = B
			// Rearrange and define Y: 
			//		Gt.X = G^-1.B = Y
			// Which gives:
			//		GY = B
			//		GtX = Y

			// Solve GY = B (G is lower-triangular) for Y
			for (int32 I = 0; I < N; ++I)
			{
				FReal Sum = B.At(I, 0);
				for (int32 K = I - 1; K >= 0; --K)
				{
					Sum -= G.At(I, K) * X.At(K, 0);
				}
				X.At(I, 0) = Sum / G.At(I, I);
			}

			// Solve GtX = Y (Gt is upper-triangular) for X
			for (int32 I = N - 1; I >= 0; --I)
			{
				FReal Sum = X.At(I, 0);
				for (int32 K = I + 1; K < N; ++K)
				{
					Sum -= G.At(K, I) * X.At(K, 0);
				}
				X.At(I, 0) = Sum / G.At(I, I);
			}
		}

		/**
		 * Solve AX = B, for positive-definite NxN matrix A, and Nx1 column vectors B and X.
		 *
		 * For positive definite A, A = GGt, where G is the Cholesky factor and lower triangular.
		 * We can solve GGtX = B by first solving GY = B, and then GtX = Y.
		 *
		 * E.g., this can be used to solve constraint equations of the form
		 *		J.I.Jt.X = B
		 * where J is a Jacobian (Jt its transpose), I is an Inverse mas matrix, and B the residual.
		 * In this case, I is symmetric positive definite, and therefore so is JIJt.
		 *
		 */
		template<int32 T_EA, int32 T_EB, int32 T_EX>
		static bool SolvePositiveDefinite(const TDenseMatrix<T_EA>& A, const TDenseMatrix<T_EB>& B, TDenseMatrix<T_EX>& X)
		{
			check(B.NumColumns() == 1);
			check(A.NumRows() == B.NumRows());

			TDenseMatrix<T_EA> G = A;
			if (!CholeskyFactorize(G))
			{
				// Not positive definite
				return false;
			}

			SolveCholeskyFactorized(G, B, X);
			return true;
		}
	};


}
