// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/ArrayCollectionArray.h"
#include "Chaos/Particles.h"

namespace Chaos
{
template<class T, int d>
class TDynamicParticles : public TParticles<T, d>
{
  public:
	TDynamicParticles()
	    : TParticles<T, d>()
	{
		TArrayCollection::AddArray(&MV);
		TArrayCollection::AddArray(&MAcceleration);
		TArrayCollection::AddArray(&MM);
		TArrayCollection::AddArray(&MInvM);
	}
	TDynamicParticles(const TDynamicParticles<T, d>& Other) = delete;
	TDynamicParticles(TDynamicParticles<T, d>&& Other)
	    : TParticles<T, d>(MoveTemp(Other)), MV(MoveTemp(Other.MV)), MAcceleration(MoveTemp(Other.MAcceleration)), MM(MoveTemp(Other.MM)), MInvM(MoveTemp(Other.MInvM))
	{
		TArrayCollection::AddArray(&MV);
		TArrayCollection::AddArray(&MAcceleration);
		TArrayCollection::AddArray(&MM);
		TArrayCollection::AddArray(&MInvM);
	}

	const TVector<T, d>& V(const int32 Index) const { return MV[Index]; }
	const TVector<T, d> GetV(const int32 Index) const { return MV[Index]; }
	TVector<T, d>& V(const int32 Index) { return MV[Index]; }
	void SetV(const int32 Index, const TVector<T, d>& InV) { MV[Index] = InV; }
	const TArrayCollectionArray<TVector<T, d>>& GetV() const { return MV; }
	TArrayCollectionArray<TVector<T, d>>& GetV() { return MV; }

	const TVector<T, d>& Acceleration(const int32 Index) const { return MAcceleration[Index]; }
	TVector<T, d>& Acceleration(const int32 Index) { return MAcceleration[Index]; }
	const TArrayCollectionArray<TVector<T, d>>& GetAcceleration() const { return MAcceleration; }
	TArrayCollectionArray<TVector<T, d>>& GetAcceleration() { return MAcceleration; }

	const T M(const int32 Index) const { return MM[Index]; }
	T& M(const int32 Index) { return MM[Index]; }
	const TArrayCollectionArray<T>& GetM() const { return MM; }
	TArrayCollectionArray<T>& GetM() { return MM; }

	const T InvM(const int32 Index) const { return MInvM[Index]; }
	T& InvM(const int32 Index) { return MInvM[Index]; }
	const TArrayCollectionArray<T>& GetInvM() const { return MInvM; }
	TArrayCollectionArray<T>& GetInvM() { return MInvM; }

  private:
	TArrayCollectionArray<TVector<T, d>> MV, MAcceleration;
	TArrayCollectionArray<T> MM, MInvM;
};

using FDynamicParticles = TDynamicParticles<FReal, 3>;
}
