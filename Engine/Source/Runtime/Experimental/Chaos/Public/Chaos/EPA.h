// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// HEADER_UNIT_SKIP - Internal

#include "Chaos/Simplex.h"
#include <queue>
#include "ChaosCheck.h"
#include "ChaosLog.h"
#include "Templates/Function.h"
#include <algorithm>

namespace Chaos
{

inline constexpr int32 ArraySizeEPA = 16;

// Array type used in EPA to avoid heap allocation for small convex shapes
// @todo(chaos): The inline size was picked to avoid allocations in box-box collision - it might need adjusting after more general purpose testing
// @todo(chaos): We might also consider different inline sizes based on array use-case (e.g., Entries array versus Border array)
template<typename T>
using TEPAWorkingArray = TArray<T, TInlineAllocator<32>>;

template <typename T>
FORCEINLINE const TVec3<T> MinkowskiVert(const TVec3<T>* VertsABuffer, const TVec3<T>* VertsBBuffer, const int32 Idx)
{
	return VertsABuffer[Idx] - VertsBBuffer[Idx];
}

template <typename T>
struct TEPAEntry
{
	int32 IdxBuffer[3];

	TVec3<T> PlaneNormal;	//Triangle normal
	T Distance;	//Triangle distance from origin
	TVector<int32,3> AdjFaces;	//Adjacent triangles
	TVector<int32,3> AdjEdges;	//Adjacent edges (idx in adjacent face)
	bool bObsolete;	//indicates that an entry can be skipped (became part of bigger polytope)

	FORCEINLINE bool operator>(const TEPAEntry<T>& Other) const
	{
		return Distance > Other.Distance;
	}

	static constexpr T SelectEpsilon(float FloatEpsilon, double DoubleEpsilon)
	{
		if (sizeof(T) <= sizeof(float))
		{
			return FloatEpsilon;
		}
		else
		{
			return DoubleEpsilon;
		}
	}

	FORCEINLINE_DEBUGGABLE bool Initialize(const TVec3<T>* VerticesA, const TVec3<T>* VerticesB, int32 InIdx0, int32 InIdx1, int32 InIdx2, const TVector<int32,3>& InAdjFaces, const TVector<int32,3>& InAdjEdges)
	{
		const TVec3<T> V0 = MinkowskiVert(VerticesA, VerticesB, InIdx0);
		const TVec3<T> V1 = MinkowskiVert(VerticesA, VerticesB, InIdx1);
		const TVec3<T> V2 = MinkowskiVert(VerticesA, VerticesB, InIdx2);

		const TVec3<T> V0V1 = V1 - V0;
		const TVec3<T> V0V2 = V2 - V0;
		const TVec3<T> Norm = TVec3<T>::CrossProduct(V0V1, V0V2);
		const T NormLenSq = Norm.SizeSquared();
		// We have the square of the size of a cross product, so we need the distance margin to be a power of 4
		// Verbosity emphasizes that
		const T Eps = TEPAEntry::SelectEpsilon(1.e-4f * 1.e-4f * 1.e-4f * 1.e-4f, 1.e-8 * 1.e-8 * 1.e-8 * 1.e-8);
		if (NormLenSq < Eps)
		{
			return false;
		}
		PlaneNormal = Norm * FMath::InvSqrt(NormLenSq);
		
		IdxBuffer[0] = InIdx0;
		IdxBuffer[1] = InIdx1;
		IdxBuffer[2] = InIdx2;

		AdjFaces = InAdjFaces;
		AdjEdges = InAdjEdges;

		Distance = TVec3<T>::DotProduct(PlaneNormal, V0);
		bObsolete = false;

		return true;
	}

	void SwapWinding(TEPAEntry* Entries)
	{
		//change vertex order
		std::swap(IdxBuffer[0], IdxBuffer[1]);

		//edges went from 0,1,2 to 1,0,2
		//0th edge/face is the same (0,1 becomes 1,0)
		//1th edge/face is now (0,2 instead of 1,2)
		//2nd edge/face is now (2,1 instead of 2,0)

		//update the adjacent face's adjacent edge first
		auto UpdateAdjEdge = [Entries, this](int32 Old, int32 New)
		{
			TEPAEntry& AdjFace = Entries[AdjFaces[Old]];
			int32& StaleAdjIdx = AdjFace.AdjEdges[AdjEdges[Old]];
			check(StaleAdjIdx == Old);
			StaleAdjIdx = New;
		};
		
		UpdateAdjEdge(1, 2);
		UpdateAdjEdge(2, 1);
		
		//now swap the actual edges and faces
		std::swap(AdjFaces[1], AdjFaces[2]);
		std::swap(AdjEdges[1], AdjEdges[2]);

		PlaneNormal = -PlaneNormal;
		Distance = -Distance;
	}

	FORCEINLINE_DEBUGGABLE T DistanceToPlane(const TVec3<T>& X) const
	{
		return TVec3<T>::DotProduct(PlaneNormal, X) - Distance;
	}

	bool IsOriginProjectedInside(const TVec3<T>* VertsABuffer, const TVec3<T>* VertsBBuffer, const T Epsilon) const
	{
		//Compare the projected point (PlaneNormal) to the triangle in the plane
		const TVec3<T> PN = Distance * PlaneNormal;
		const TVec3<T> PA = MinkowskiVert(VertsABuffer, VertsBBuffer, IdxBuffer[0]) - PN;
		const TVec3<T> PB = MinkowskiVert(VertsABuffer, VertsBBuffer, IdxBuffer[1]) - PN;
		const TVec3<T> PC = MinkowskiVert(VertsABuffer, VertsBBuffer, IdxBuffer[2]) - PN;

		const TVec3<T> PACNormal = TVec3<T>::CrossProduct(PA, PC);
		const T PACSign = TVec3<T>::DotProduct(PACNormal, PlaneNormal);
		const TVec3<T> PCBNormal = TVec3<T>::CrossProduct(PC, PB);
		const T PCBSign = TVec3<T>::DotProduct(PCBNormal, PlaneNormal);

		if((PACSign < -Epsilon && PCBSign > Epsilon) || (PACSign > Epsilon && PCBSign < -Epsilon))
		{
			return false;
		}

		const TVec3<T> PBANormal = TVec3<T>::CrossProduct(PB, PA);
		const T PBASign = TVec3<T>::DotProduct(PBANormal, PlaneNormal);

		if((PACSign < -Epsilon && PBASign > Epsilon) || (PACSign > Epsilon && PBASign < -Epsilon))
		{
			return false;
		}

		return true;
	}
};

template <typename T>
bool InitializeEPA(TArray<TVec3<T>>& VertsA, TArray<TVec3<T>>& VertsB, TFunctionRef<TVector<T, 3>(const TVec3<T>& V)> SupportA, TFunctionRef<TVector<T, 3>(const TVec3<T>& V)> SupportB, TEPAWorkingArray<TEPAEntry<T>>& OutEntries, TVec3<T>& OutTouchNormal)
{
	const int32 NumVerts = VertsA.Num();
	check(VertsB.Num() == NumVerts);

	auto AddFartherPoint = [&](const TVec3<T>& Dir)
	{
		const TVec3<T> NegDir = -Dir;
		const TVec3<T> A0 = SupportA(Dir);	//should we have a function that does both directions at once?
		const TVec3<T> A1 = SupportA(NegDir);
		const TVec3<T> B0 = SupportB(NegDir);
		const TVec3<T> B1 = SupportB(Dir);

		const TVec3<T> W0 = A0 - B0;
		const TVec3<T> W1 = A1 - B1;

		const T Dist0 = TVec3<T>::DotProduct(W0, Dir);
		const T Dist1 = TVec3<T>::DotProduct(W1, NegDir);

		if (Dist1 >= Dist0)
		{
			VertsA.Add(A1);
			VertsB.Add(B1);
		}
		else
		{
			VertsA.Add(A0);
			VertsB.Add(B0);
		}
	};

	OutEntries.AddUninitialized(4);
	OutTouchNormal = TVec3<T>(0,0,1);

	bool bValid = false;

	switch(NumVerts)
	{
		case 1:
		{
			//assuming it's a touching hit at origin, but we still need to calculate a separating normal
			AddFartherPoint(OutTouchNormal); // Use an arbitrary direction

			// Now we might have a line! So fall trough to the next case
		}
		case 2:
		{
			//line, add farthest point along most orthogonal axes
			TVec3<T> Dir = MinkowskiVert(VertsA.GetData(), VertsB.GetData(), 1) - MinkowskiVert(VertsA.GetData(), VertsB.GetData(), 0);

			bValid = Dir.SizeSquared() > 1e-4;
			if (bValid)	//two verts given are distinct
			{
				//find most opposing axis
				int32 BestAxis = 0;
				T MinVal = TNumericLimits<T>::Max();
				for (int32 Axis = 0; Axis < 3; ++Axis)
				{
					const T AbsVal = FMath::Abs(Dir[Axis]);
					if (MinVal > AbsVal)
					{
						BestAxis = Axis;
						MinVal = AbsVal;
					}
				}
				const TVec3<T> OtherAxis = TVec3<T>::AxisVector(BestAxis);
				const TVec3<T> Orthog = TVec3<T>::CrossProduct(Dir, OtherAxis);
				const TVec3<T> Orthog2 = TVec3<T>::CrossProduct(Orthog, Dir);

				AddFartherPoint(Orthog);
				AddFartherPoint(Orthog2);

				bValid = OutEntries[0].Initialize(VertsA.GetData(), VertsB.GetData(), 1, 2, 3, { 3,1,2 }, { 1,1,1 });
				bValid &= OutEntries[1].Initialize(VertsA.GetData(), VertsB.GetData(), 0, 3, 2, { 2,0,3 }, { 2,1,0 });
				bValid &= OutEntries[2].Initialize(VertsA.GetData(), VertsB.GetData(), 0, 1, 3, { 3,0,1 }, { 2,2,0 });
				bValid &= OutEntries[3].Initialize(VertsA.GetData(), VertsB.GetData(), 0, 2, 1, { 1,0,2 }, { 2,0,0 });

				if(!bValid)
				{
					OutTouchNormal = Orthog.GetUnsafeNormal();
					return false;
				}
			}
			else
			{
				// The two vertices are not distinct that may happen when the single vertex case above was hit and our CSO is very thin in that direction
				CHAOS_ENSURE(NumVerts == 1); // If this ensure fires we were given 2 vertices that are not distinct to start with
				return false;
			}
			break;
		}
		case 3:
		{
			//triangle, add farthest point along normal
			bValid = OutEntries[3].Initialize(VertsA.GetData(), VertsB.GetData(), 0, 2, 1, { 1,0,2 }, { 2,0,0 });	
			if (CHAOS_ENSURE(bValid)) //input verts must form a valid triangle
			{
				const TEPAEntry<T>& Base = OutEntries[3];

				AddFartherPoint(Base.PlaneNormal);

				bValid = OutEntries[0].Initialize(VertsA.GetData(), VertsB.GetData(), 1, 2, 3, { 3,1,2 }, { 1,1,1 });
				bValid &= OutEntries[1].Initialize(VertsA.GetData(), VertsB.GetData(), 0, 3, 2, { 2,0,3 }, { 2,1,0 });
				bValid &= OutEntries[2].Initialize(VertsA.GetData(), VertsB.GetData(), 0, 1, 3, { 3,0,1 }, { 2,2,0 });

				if(!bValid)
				{
					OutTouchNormal = Base.PlaneNormal;
					return false;
				}
			}
			break;
		}
		case 4:
		{
			bValid = OutEntries[0].Initialize(VertsA.GetData(), VertsB.GetData(), 1, 2, 3, { 3,1,2 }, { 1,1,1 });
			bValid &= OutEntries[1].Initialize(VertsA.GetData(), VertsB.GetData(), 0, 3, 2, { 2,0,3 }, { 2,1,0 });
			bValid &= OutEntries[2].Initialize(VertsA.GetData(), VertsB.GetData(), 0, 1, 3, { 3,0,1 }, { 2,2,0 });
			bValid &= OutEntries[3].Initialize(VertsA.GetData(), VertsB.GetData(), 0, 2, 1, { 1,0,2 }, { 2,0,0 });
			
			if(!bValid)
			{
				CHAOS_ENSURE(false);	//expect user to give us valid tetrahedron
				UE_LOGF(LogChaos, Log, "Invalid tetrahedron encountered in InitializeEPA");
			}

			break;
		}

		default: 
		{
			CHAOS_ENSURE(false);
			break;
		}
	}

	if (bValid)
	{
		//make sure normals are pointing out of tetrahedron
		// In the usual case the distances will either all be positive or negative, 
		// but the tetrahedron can be very close to (or touching) the origin
		// Look for farthest plane to decide
		T MaxSignedDistance = 0;
		for (TEPAEntry<T>& Entry : OutEntries)
		{
			if (FMath::Abs(Entry.Distance) > FMath::Abs(MaxSignedDistance))
			{
				MaxSignedDistance = Entry.Distance;
			}
		}

		if (MaxSignedDistance < 0.0f)
		{
			for (TEPAEntry<T>& Entry : OutEntries)
			{
				Entry.SwapWinding(OutEntries.GetData());
			}
		}
	}
	
	return bValid;
}

template <typename T, typename SupportALambda, typename SupportBLambda >
bool InitializeEPA(TArray<TVec3<T>>& VertsA, TArray<TVec3<T>>& VertsB, const SupportALambda& SupportA, const SupportBLambda& SupportB, TEPAWorkingArray<TEPAEntry<T>>& OutEntries, TVec3<T>& OutTouchNormal)
{
	TFunctionRef<TVector<T, 3>(const TVec3<T>& V)> SupportARef(SupportA);
	TFunctionRef<TVector<T, 3>(const TVec3<T>& V)> SupportBRef(SupportB);

	return InitializeEPA<T>(VertsA, VertsB, SupportARef, SupportBRef, OutEntries, OutTouchNormal);
}

struct FEPAFloodEntry
{
	int32 EntryIdx;
	int32 EdgeIdx;
};

template <typename T>
void EPAComputeVisibilityBorder(TEPAWorkingArray<TEPAEntry<T>>& Entries, int32 EntryIdx, const TVec3<T>& W, TEPAWorkingArray<FEPAFloodEntry>& OutBorderEdges, TEPAWorkingArray<FEPAFloodEntry> &ToVisitStack)
{
	{
		TEPAEntry<T>& Entry = Entries[EntryIdx];
		for (int i = 0; i < 3; ++i)
		{
			ToVisitStack.Add({ Entry.AdjFaces[i], Entry.AdjEdges[i] });
		}
	}

	int32 Iteration = 0;
	const int32 MaxIteration = 10000;

	while (ToVisitStack.Num() && Iteration++ < MaxIteration)
	{
		const FEPAFloodEntry FloodEntry = ToVisitStack.Pop(EAllowShrinking::No);
		TEPAEntry<T>& Entry = Entries[FloodEntry.EntryIdx];
		if (!Entry.bObsolete)
		{
			if (Entry.DistanceToPlane(W) <= TEPAEntry<T>::SelectEpsilon(1.e-4f, 1.e-8))
			{
				//W can't see this triangle so mark the edge as a border
				OutBorderEdges.Add(FloodEntry);
			}
			else
			{
				//W can see this triangle so continue flood fill
				Entry.bObsolete = true;	//no longer needed
				const int32 Idx0 = FloodEntry.EdgeIdx;
				const int32 Idx1 = (Idx0 + 1) % 3;
				const int32 Idx2 = (Idx0 + 2) % 3;
				ToVisitStack.Add({ Entry.AdjFaces[Idx1], Entry.AdjEdges[Idx1] });
				ToVisitStack.Add({ Entry.AdjFaces[Idx2], Entry.AdjEdges[Idx2] });
			}
		}
	}

	if(Iteration >= MaxIteration)
	{
		UE_LOGF(LogChaos,Warning,"EPAComputeVisibilityBorder reached max iteration - something is wrong");
	}
}

template <typename T>
void ComputeEPAResults(const TVec3<T>* VertsA, const TVec3<T>* VertsB, const TEPAEntry<T>& Entry, T& OutPenetration, TVec3<T>& OutDir, TVec3<T>& OutA, TVec3<T>& OutB)
{
	//NOTE: We use this function as fallback when robustness breaks. So - do not assume adjacency is valid as these may be new uninitialized traingles that failed
	FSimplex SimplexIDs({ 0,1,2 });
	TVec3<T> As[4] = { VertsA[Entry.IdxBuffer[0]], VertsA[Entry.IdxBuffer[1]], VertsA[Entry.IdxBuffer[2]] };
	TVec3<T> Bs[4] = { VertsB[Entry.IdxBuffer[0]], VertsB[Entry.IdxBuffer[1]], VertsB[Entry.IdxBuffer[2]] };
	TVec3<T> Simplex[4] = { As[0] - Bs[0], As[1] - Bs[1], As[2] - Bs[2] };
	T Barycentric[4];

	OutDir = SimplexFindClosestToOrigin(Simplex, SimplexIDs, Barycentric, As, Bs);
	OutPenetration = OutDir.Size();

	// @todo(chaos): pass in epsilon? Does it need to match anything in GJK?
	if (OutPenetration < 1e-4)	//if closest point is on the origin (edge case when surface is right on the origin)
	{
		OutDir = Entry.PlaneNormal;	//just fall back on plane normal
		if (Entry.Distance < 0)
		{
			OutPenetration = -OutPenetration; // We are a bit outside of the shape so penetration is negative
		}
	}
	else
	{
		OutDir /= OutPenetration;
		if (Entry.Distance < 0)
		{
			//The origin is on the outside, so the direction is reversed
			OutDir = -OutDir;
			OutPenetration = -OutPenetration; // We are a bit outside of the shape so penetration is negative
		}
	}

	OutA = TVec3<T>(0);
	OutB = TVec3<T>(0);

	for (int i = 0; i < SimplexIDs.NumVerts; ++i)
	{
		OutA += As[i] * Barycentric[i];
		OutB += Bs[i] * Barycentric[i];
	}
}

enum class EEPAResult
{
	Ok,							// Successfully found the contact point to within the tolerance
	MaxIterations,				// We have a contact point, but did not reach the target tolerance before we hit the iteration limit - result accuracy is unknown
	Degenerate,					// We hit a degenerate condition in EPA which prevents a solution from being generated (result is invalid but objects may be penetrating)
	BadInitialSimplex,			// The initial setup did not provide a polytope containing the origin (objects are separated)
	NoValidContact,             // No valid contact have been found, no contacts will be returned
};

UE_DEPRECATED(5.3, "Not used")
inline const bool IsEPASuccess(EEPAResult EPAResult)
{
	return (EPAResult == EEPAResult::Ok) || (EPAResult == EEPAResult::MaxIterations);
}

#ifndef DEBUG_EPA
#define DEBUG_EPA 0
#endif

// Expanding Polytope Algorithm for finding the contact point for overlapping convex polyhedra.
// See e.g., "Collision Detection in Interactive 3D Environments" (Gino van den Bergen, 2004)
// or "Real-time Collision Detection with Implicit Objects" (Leif Olvang, 2010)
template <typename T, typename TSupportA, typename TSupportB>
EEPAResult EPA(TArray<TVec3<T>>& VertsABuffer, TArray<TVec3<T>>& VertsBBuffer, const TSupportA& SupportA, const TSupportB& SupportB, T& OutPenetration, TVec3<T>& OutDir, TVec3<T>& WitnessA, TVec3<T>& WitnessB, const FReal Eps = 1.e-2f)
{
	const TFunctionRef<TVector<T, 3>(const TVec3<T>& V)> SupportFuncA(SupportA);
	const TFunctionRef<TVector<T, 3>(const TVec3<T>& V)> SupportFuncB(SupportB);
	return EPA<T>(VertsABuffer, VertsBBuffer, SupportFuncA, SupportFuncB, OutPenetration, OutDir, WitnessA, WitnessB, Eps);
}

template <typename T>
EEPAResult EPA(TArray<TVec3<T>>& VertsABuffer, TArray<TVec3<T>>& VertsBBuffer, const TFunctionRef<TVector<T, 3>(const TVec3<T>& V)>& SupportA,
	const TFunctionRef<TVector<T, 3>(const TVec3<T>& V)>& SupportB, T& OutPenetration, TVec3<T>& OutDir, TVec3<T>& WitnessA, TVec3<T>& WitnessB, const FReal EpsRel = 1.e-2f)
{
	struct FEPAEntryWrapper
	{
		const TArray<TEPAEntry<T>>* Entries;		
		int32 Idx;

		bool operator>(const FEPAEntryWrapper& Other) const
		{
			return (*Entries)[Idx] > (*Entries)[Other.Idx];
		}
	};

	constexpr T OriginInsideEps = 0.0f;

	TEPAWorkingArray<TEPAEntry<T>> Entries;

	if(!InitializeEPA(VertsABuffer,VertsBBuffer,SupportA,SupportB, Entries, OutDir))
	{
		//either degenerate or a touching hit. Either way return penetration 0
		OutPenetration = 0;
		WitnessA = TVec3<T>(0);
		WitnessB = TVec3<T>(0);
		return EEPAResult::BadInitialSimplex;
	}

#if DEBUG_EPA
	TArray<TVec3<T>> VertsWBuffer;
	for (int32 Idx = 0; Idx < 4; ++Idx)
	{
		VertsWBuffer.Add(MinkowskiVert(VertsABuffer.GetData(), VertsBBuffer.GetData(), Idx));
	}
#endif
	
	TEPAWorkingArray<int32> Queue;
	for(int32 Idx = 0; Idx < Entries.Num(); ++Idx)
	{
		//ensure(Entries[Idx].Distance > -Eps);
		// Entries[Idx].Distance <= 0.0f is true if the origin is a bit out of the polytope (we need to support this case for robustness)
		if(Entries[Idx].Distance <= 0.0f || Entries[Idx].IsOriginProjectedInside(VertsABuffer.GetData(), VertsBBuffer.GetData(), OriginInsideEps))
		{
			Queue.Add(Idx);
		}
	}

	TEPAWorkingArray<FEPAFloodEntry> VisibilityBorder;
	TEPAWorkingArray<FEPAFloodEntry> VisibilityBorderToVisitStack;

	//TEPAEntry<T> BestEntry;
	//BestEntry.Distance = 0;
	TEPAEntry<T> LastEntry = Queue.Num() > 0 ? Entries[Queue.Last()] : Entries[0];
	T UpperBound = TNumericLimits<T>::Max();
	T LowerBound = TNumericLimits<T>::Lowest();
	bool bQueueDirty = true;
	int32 Iteration = 0;
	int32 constexpr MaxIterations = 128;
	EEPAResult ResultStatus = EEPAResult::MaxIterations;
	while (Queue.Num() && (Iteration++ < MaxIterations))
	{
		if (bQueueDirty)
		{
			// Avoiding UE's Sort here because it has a call to FMath::Loge. The std version calls insertion sort when possible
			std::sort(Queue.GetData(), Queue.GetData() + Queue.Num(), [&Entries](const int32 L, const int32 R) { return Entries[L] > Entries[R]; });
			bQueueDirty = false;
		}

		int32 EntryIdx = Queue.Pop(EAllowShrinking::No);
		TEPAEntry<T>& Entry = Entries[EntryIdx];
		//bool bBadFace = Entry.IsOriginProjectedInside(VertsABuffer.GetData(), VertsBBuffer.GetData());
		{
			//UE_LOGF(LogChaos, Warning, "%d BestW:%f, Distance:%f, bObsolete:%d, InTriangle:%d",
			//	Iteration, BestW.Size(), Entry.Distance, Entry.bObsolete, bBadFace);
		}
		if (Entry.bObsolete)
		{
			// @todo(chaos): should this count as an iteration? Currently it does...
			continue;
		}

		if (Entry.Distance > UpperBound)
		{
			ResultStatus = EEPAResult::Ok;
			break;
		}

		const TVec3<T> ASupport = SupportA(Entry.PlaneNormal);
		const TVec3<T> BSupport = SupportB(-Entry.PlaneNormal);
		const TVec3<T> W = ASupport - BSupport;
		const T DistanceToSupportPlane = TVec3<T>::DotProduct(Entry.PlaneNormal, W);
		if(DistanceToSupportPlane < UpperBound)
		{
			UpperBound = DistanceToSupportPlane;
			//Remember the entry that gave us the lowest upper bound and use it in case we have to terminate early
			//This can result in very deep planes. Ideally we'd just use the plane formed at W, but it's not clear how you get back points in A, B for that
			//BestEntry = Entry;
		}

		LowerBound = Entry.Distance;

		// It's possible the origin is not contained by the CSO, probably because of numerical error. 
		// In this case the upper bound will be negative, at which point we should just exit. 
		const T UpperBoundTolerance = (T(1) + EpsRel) * FMath::Abs(LowerBound);
		if (UpperBound <= UpperBoundTolerance)
		{
			ResultStatus = EEPAResult::Ok;
			LastEntry = Entry;
			break;
		}

		if (UpperBound < LowerBound)
		{
			//we cannot get any better than what we saw, so just return previous face
			ResultStatus = EEPAResult::Ok;
			break;
		}

		LastEntry = Entry;


		const int32 NewVertIdx = VertsABuffer.Add(ASupport);
		VertsBBuffer.Add(BSupport);

#if DEBUG_EPA
		VertsWBuffer.Add(MinkowskiVert(VertsABuffer.GetData(), VertsBBuffer.GetData(), NewVertIdx));
#endif

		Entry.bObsolete = true;
		VisibilityBorder.Reset();
		VisibilityBorderToVisitStack.Reset();
		EPAComputeVisibilityBorder(Entries, EntryIdx, W, VisibilityBorder, VisibilityBorderToVisitStack);
		const int32 NumBorderEdges = VisibilityBorder.Num();
		const int32 FirstIdxInBatch = Entries.Num();
		int32 NewIdx = FirstIdxInBatch;
		Entries.AddUninitialized(NumBorderEdges);
		if (NumBorderEdges >= 3)
		{
			bool bTerminate = false;
			for (int32 VisibilityIdx = 0; VisibilityIdx < NumBorderEdges; ++VisibilityIdx)
			{
				//create new entries and update adjacencies
				const FEPAFloodEntry& BorderInfo = VisibilityBorder[VisibilityIdx];
				TEPAEntry<T>& NewEntry = Entries[NewIdx];
				const int32 BorderEntryIdx = BorderInfo.EntryIdx;
				TEPAEntry<T>& BorderEntry = Entries[BorderEntryIdx];
				const int32 BorderEdgeIdx0 = BorderInfo.EdgeIdx;
				const int32 BorderEdgeIdx1 = (BorderEdgeIdx0 + 1) % 3;
				const int32 NextEntryIdx = (VisibilityIdx + 1) < VisibilityBorder.Num() ? NewIdx + 1 : FirstIdxInBatch;
				const int32 PrevEntryIdx = NewIdx > FirstIdxInBatch ? NewIdx - 1 : FirstIdxInBatch + NumBorderEdges - 1;
				const bool bValidTri = NewEntry.Initialize(VertsABuffer.GetData(), VertsBBuffer.GetData(), BorderEntry.IdxBuffer[BorderEdgeIdx1], BorderEntry.IdxBuffer[BorderEdgeIdx0], NewVertIdx,
					{ BorderEntryIdx, PrevEntryIdx, NextEntryIdx },
					{ BorderEdgeIdx0, 2, 1 });
				BorderEntry.AdjFaces[BorderEdgeIdx0] = NewIdx;
				BorderEntry.AdjEdges[BorderEdgeIdx0] = 0;
				
				if (!bValidTri)
				{
					//couldn't properly expand polytope, so just stop
					ResultStatus = EEPAResult::Degenerate;
					bTerminate = true;
					break;
				}

				// Due to numerical inaccuracies NewEntry.Distance >= LowerBound may be false!
				// However these Entries still have good normals, and needs to be included to prevent
				// this exiting with very deep penetration results
				
				if (bValidTri && NewEntry.Distance <= UpperBound)
				{
					// NewEntry.Distance <= 0.0f is if the origin is a bit out of the polytope
					if (NewEntry.Distance <= 0.0f || NewEntry.IsOriginProjectedInside(VertsABuffer.GetData(), VertsBBuffer.GetData(), OriginInsideEps))
					{
						Queue.Add(NewIdx);
						bQueueDirty = true;
					}
				}

				++NewIdx;
			}

			if (bTerminate)
			{
				break;
			}
		}
		else
		{
			//couldn't properly expand polytope, just stop now
			ResultStatus = EEPAResult::Degenerate;
			break;
		}
	}

	ComputeEPAResults(VertsABuffer.GetData(), VertsBBuffer.GetData(), LastEntry, OutPenetration, OutDir, WitnessA, WitnessB);
	
	return ResultStatus;
}

}
