// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/SimplexVectorized.h"
#include "Chaos/EPA.h"

#include <queue>
#include "ChaosCheck.h"
#include "ChaosLog.h"
#include "Templates/Function.h"

#include "Math/VectorRegister.h"

namespace Chaos
{


FORCEINLINE const VectorRegister4Float VectorMinkowskiVert(const VectorRegister4Float* VertsABuffer, const VectorRegister4Float* VertsBBuffer, const int32 Idx)
{
	return VectorSubtract(VertsABuffer[Idx], VertsBBuffer[Idx]);
}


struct VectorTEPAEntry
{
	VectorRegister4Float PlaneNormal;	//Triangle normal
	VectorRegister4Float Distance;	//Triangle distance from origin
	int32 IdxBuffer[3];
	TVector<int32, 3> AdjFaces;	//Adjacent triangles
	TVector<int32, 3> AdjEdges;	//Adjacent edges (idx in adjacent face)
	bool bObsolete;	//indicates that an entry can be skipped (became part of bigger polytope)

	FORCEINLINE bool operator>(const VectorTEPAEntry& Other) const
	{
		return static_cast<bool>(::VectorMaskBits(VectorCompareGT(Distance, Other.Distance)));
	}

	FORCEINLINE_DEBUGGABLE bool Initialize(const VectorRegister4Float* VerticesA, const VectorRegister4Float* VerticesB, int32 InIdx0, int32 InIdx1, int32 InIdx2, const TVector<int32, 3>& InAdjFaces, const TVector<int32, 3>& InAdjEdges)
	{
		const VectorRegister4Float V0 = VectorMinkowskiVert(VerticesA, VerticesB, InIdx0);
		const VectorRegister4Float V1 = VectorMinkowskiVert(VerticesA, VerticesB, InIdx1);
		const VectorRegister4Float V2 = VectorMinkowskiVert(VerticesA, VerticesB, InIdx2);

		const VectorRegister4Float V0V1 = VectorSubtract(V1, V0);
		const VectorRegister4Float V0V2 = VectorSubtract(V2, V0);
		const VectorRegister4Float Norm = VectorCross(V0V1, V0V2);
		const VectorRegister4Float NormLenSq = VectorDot3(Norm, Norm);
		constexpr VectorRegister4Float Eps = MakeVectorRegisterFloatConstant(1e-4f, 1e-4f, 1e-4f, 1e-4f);
		const VectorRegister4Float EpsGTNormLenSq = VectorCompareGT(Eps, NormLenSq);
		if (::VectorMaskBits(EpsGTNormLenSq))
		{
			return false;
		}
		PlaneNormal = VectorMultiply(Norm, VectorReciprocalSqrt(NormLenSq));

		IdxBuffer[0] = InIdx0;
		IdxBuffer[1] = InIdx1;
		IdxBuffer[2] = InIdx2;

		AdjFaces = InAdjFaces;
		AdjEdges = InAdjEdges;

		Distance = VectorDot3(PlaneNormal, V0);
		bObsolete = false;

		return true;
	}

	void SwapWinding(VectorTEPAEntry* Entries)
	{
		//change vertex order
		std::swap(IdxBuffer[0], IdxBuffer[1]);

		//edges went from 0,1,2 to 1,0,2
		//0th edge/face is the same (0,1 becomes 1,0)
		//1th edge/face is now (0,2 instead of 1,2)
		//2nd edge/face is now (2,1 instead of 2,0)

		//update the adjacent face's adjacent edge first
		auto UpdateAdjEdge = [Entries, this](int32 Old, int32 New)
		{
			VectorTEPAEntry& AdjFace = Entries[AdjFaces[Old]];
			int32& StaleAdjIdx = AdjFace.AdjEdges[AdjEdges[Old]];
			check(StaleAdjIdx == Old);
			StaleAdjIdx = New;
		};

		UpdateAdjEdge(1, 2);
		UpdateAdjEdge(2, 1);

		//now swap the actual edges and faces
		std::swap(AdjFaces[1], AdjFaces[2]);
		std::swap(AdjEdges[1], AdjEdges[2]);

		PlaneNormal = VectorNegate(PlaneNormal);
		Distance = VectorNegate(Distance);
	}

	FORCEINLINE_DEBUGGABLE VectorRegister4Float DistanceToPlane(const VectorRegister4Float& X) const
	{
		return VectorSubtract(VectorDot3(PlaneNormal, X), Distance);
	}

	FORCEINLINE_DEBUGGABLE bool IsOriginProjectedInside(const VectorRegister4Float* VertsABuffer, const VectorRegister4Float* VertsBBuffer, const VectorRegister4Float Epsilon) const
	{
		//Compare the projected point (PlaneNormal) to the triangle in the plane
		const VectorRegister4Float PN = VectorMultiply(Distance, PlaneNormal);
		const VectorRegister4Float PA = VectorSubtract(VectorMinkowskiVert(VertsABuffer, VertsBBuffer, IdxBuffer[0]), PN);
		const VectorRegister4Float PB = VectorSubtract(VectorMinkowskiVert(VertsABuffer, VertsBBuffer, IdxBuffer[1]), PN);
		const VectorRegister4Float PC = VectorSubtract(VectorMinkowskiVert(VertsABuffer, VertsBBuffer, IdxBuffer[2]), PN);

		const VectorRegister4Float PACNormal = VectorCross(PA, PC);
		const VectorRegister4Float PACSign = VectorDot3(PACNormal, PlaneNormal);
		const VectorRegister4Float PCBNormal = VectorCross(PC, PB);
		const VectorRegister4Float PCBSign = VectorDot3(PCBNormal, PlaneNormal);


		const VectorRegister4Float MinusEps = VectorNegate(Epsilon);
		const VectorRegister4Float MinusEpsGTPACSign = VectorCompareGT(MinusEps, PACSign);
		const VectorRegister4Float PCBSignGTEps = VectorCompareGT(PCBSign, Epsilon);
		const VectorRegister4Float PACSignGTEps = VectorCompareGT(PACSign, Epsilon);
		const VectorRegister4Float MinusEpsGTPCBSign = VectorCompareGT(MinusEps, PCBSign);

		const VectorRegister4Float InterACCB = VectorBitwiseAnd(MinusEpsGTPACSign, PCBSignGTEps);
		const VectorRegister4Float InterCBAC = VectorBitwiseAnd(PACSignGTEps, MinusEpsGTPCBSign);
		const VectorRegister4Float IsFalse1 = VectorBitwiseOr(InterACCB, InterCBAC);


		// (PACSign < -Epsilon && PCBSign > Epsilon) || (PACSign > Epsilon && PCBSign < -Epsilon)
		
		const VectorRegister4Float PBANormal = VectorCross(PB, PA);
		const VectorRegister4Float PBASign = VectorDot3(PBANormal, PlaneNormal);

		const VectorRegister4Float PBASignGTEps = VectorCompareGT(PBASign, Epsilon);
		const VectorRegister4Float MinusEpsGTPBASign = VectorCompareGT(MinusEps, PBASign);

		const VectorRegister4Float InterACBA = VectorBitwiseAnd(MinusEpsGTPACSign, PBASignGTEps);
		const VectorRegister4Float InterBAAC = VectorBitwiseAnd(PACSignGTEps, MinusEpsGTPBASign);
		const VectorRegister4Float IsFalse2 = VectorBitwiseOr(InterACBA, InterBAAC);

		const VectorRegister4Float IsFalse = VectorBitwiseOr(IsFalse1, IsFalse2);

		return !static_cast<bool>(::VectorMaskBits(IsFalse));
	}
};
template <typename SupportALambda, typename SupportBLambda >
bool VectorInitializeEPA(TArray<VectorRegister4Float>& VertsA, TArray<VectorRegister4Float>& VertsB, const SupportALambda& SupportA, const SupportBLambda& SupportB, TEPAWorkingArray<VectorTEPAEntry>& OutEntries, VectorRegister4Float& OutTouchNormal)
{
	const int32 NumVerts = VertsA.Num();
	check(VertsB.Num() == NumVerts);

	auto AddFartherPoint = [&](const VectorRegister4Float& Dir)
	{
		const VectorRegister4Float NegDir = VectorNegate(Dir);
		const VectorRegister4Float A0 = SupportA(Dir);	//should we have a function that does both directions at once?
		const VectorRegister4Float A1 = SupportA(NegDir);
		const VectorRegister4Float B0 = SupportB(NegDir);
		const VectorRegister4Float B1 = SupportB(Dir);

		const VectorRegister4Float W0 = VectorSubtract(A0, B0);
		const VectorRegister4Float W1 = VectorSubtract(A1, B1);

		const VectorRegister4Float Dist0 = VectorDot3(W0, Dir);
		const VectorRegister4Float Dist1 = VectorDot3(W1, NegDir);

		const VectorRegister4Float IsDist1GEDist0 = VectorCompareGE(Dist1, Dist0);

		if (::VectorMaskBits(IsDist1GEDist0))
		{
			VertsA.Add(A1);
			VertsB.Add(B1);
		}
		else
		{
			VertsA.Add(A0);
			VertsB.Add(B0);
		}
	};

	OutEntries.AddUninitialized(4);
	OutTouchNormal = MakeVectorRegisterFloat(0.0f, 0.0f, 1.0f, 0.0f);

	bool bValid = false;

	switch (NumVerts)
	{
	case 1:
	{
		//assuming it's a touching hit at origin, but we still need to calculate a separating normal
		AddFartherPoint(OutTouchNormal); // Use an arbitrary direction

		// Now we might have a line! So fall trough to the next case
	}
	case 2:
	{
		//line, add farthest point along most orthogonal axes
		const VectorRegister4Float Dir = VectorSubtract(VectorMinkowskiVert(VertsA.GetData(), VertsB.GetData(), 1),  VectorMinkowskiVert(VertsA.GetData(), VertsB.GetData(), 0));

		constexpr VectorRegister4Float Eps = MakeVectorRegisterFloatConstant(1e-4f, 1e-4f, 1e-4f, 1e-4f);
		const VectorRegister4Float Valid = VectorCompareGT(VectorDot3(Dir, Dir),  Eps);
		if (::VectorMaskBits(Valid))	//two verts given are distinct
		{
			// return MakeVectorRegisterFloatFromDouble(LoadFloat3(Center));
			const VectorRegister4Float DirAbs = VectorAbs(Dir);
			const VectorRegister4Float DirAbs0 = VectorSwizzle(DirAbs, 0, 0, 0, 0);
			const VectorRegister4Float DirAbs1 = VectorSwizzle(DirAbs, 1, 1, 1, 1);
			const VectorRegister4Float DirAbs2 = VectorSwizzle(DirAbs, 2, 2, 2, 2);

			const VectorRegister4Float Dir0GTDir1 = VectorCompareGT(DirAbs0, DirAbs1);
			const VectorRegister4Float Dir0GTDir2 = VectorCompareGT(DirAbs0, DirAbs2);
			const VectorRegister4Float Dir1GTDir2 = VectorCompareGT(DirAbs1, DirAbs2);

			constexpr VectorRegister4Float Axis1 = MakeVectorRegisterFloatConstant(0.0f, 1.0f, 0.0f, 0.0f);
			constexpr VectorRegister4Float Axis2 = MakeVectorRegisterFloatConstant(0.0f, 0.0f, 1.0f, 0.0f);

			const VectorRegister4Float Axis01 = VectorSelect(Dir0GTDir1, Axis1, GlobalVectorConstants::Float1000);
			const VectorRegister4Float Axis02 = VectorSelect(Dir0GTDir2, Axis2, GlobalVectorConstants::Float1000);
			const VectorRegister4Float OtherAxis = VectorSelect(Dir1GTDir2, Axis02, Axis01);

			const VectorRegister4Float Orthog = VectorCross(Dir, OtherAxis);
			const VectorRegister4Float Orthog2 = VectorCross(Orthog, Dir);

			AddFartherPoint(Orthog);
			AddFartherPoint(Orthog2);

			bValid = OutEntries[0].Initialize(VertsA.GetData(), VertsB.GetData(), 1, 2, 3, { 3,1,2 }, { 1,1,1 });
			bValid &= OutEntries[1].Initialize(VertsA.GetData(), VertsB.GetData(), 0, 3, 2, { 2,0,3 }, { 2,1,0 });
			bValid &= OutEntries[2].Initialize(VertsA.GetData(), VertsB.GetData(), 0, 1, 3, { 3,0,1 }, { 2,2,0 });
			bValid &= OutEntries[3].Initialize(VertsA.GetData(), VertsB.GetData(), 0, 2, 1, { 1,0,2 }, { 2,0,0 });

			if (!bValid)
			{
				OutTouchNormal = VectorNormalizeAccurate(Orthog);
				return false;
			}
		}
		else
		{
			// The two vertices are not distinct that may happen when the single vertex case above was hit and our CSO is very thin in that direction
			CHAOS_ENSURE(NumVerts == 1); // If this ensure fires we were given 2 vertices that are not distinct to start with
			return false;
		}
		break;
	}
	case 3:
	{
		//triangle, add farthest point along normal
		bValid = OutEntries[3].Initialize(VertsA.GetData(), VertsB.GetData(), 0, 2, 1, { 1,0,2 }, { 2,0,0 });
		if (CHAOS_ENSURE(bValid)) //input verts must form a valid triangle
		{
			const VectorTEPAEntry& Base = OutEntries[3];

			AddFartherPoint(Base.PlaneNormal);

			bValid = OutEntries[0].Initialize(VertsA.GetData(), VertsB.GetData(), 1, 2, 3, { 3,1,2 }, { 1,1,1 });
			bValid &= OutEntries[1].Initialize(VertsA.GetData(), VertsB.GetData(), 0, 3, 2, { 2,0,3 }, { 2,1,0 });
			bValid &= OutEntries[2].Initialize(VertsA.GetData(), VertsB.GetData(), 0, 1, 3, { 3,0,1 }, { 2,2,0 });

			if (!bValid)
			{
				OutTouchNormal = Base.PlaneNormal;
				return false;
			}
		}
		break;
	}
	case 4:
	{
		bValid = OutEntries[0].Initialize(VertsA.GetData(), VertsB.GetData(), 1, 2, 3, { 3,1,2 }, { 1,1,1 });
		bValid &= OutEntries[1].Initialize(VertsA.GetData(), VertsB.GetData(), 0, 3, 2, { 2,0,3 }, { 2,1,0 });
		bValid &= OutEntries[2].Initialize(VertsA.GetData(), VertsB.GetData(), 0, 1, 3, { 3,0,1 }, { 2,2,0 });
		bValid &= OutEntries[3].Initialize(VertsA.GetData(), VertsB.GetData(), 0, 2, 1, { 1,0,2 }, { 2,0,0 });

		if (!bValid)
		{
			CHAOS_ENSURE(false);	//expect user to give us valid tetrahedron
			UE_LOGF(LogChaos, Log, "Invalid tetrahedron encountered in VectorInitializeEPA");
		}

		break;
	}

	default:
	{
		CHAOS_ENSURE(false);
		break;
	}
	}

	if (bValid)
	{
		//make sure normals are pointing out of tetrahedron
		// In the usual case the distances will either all be positive or negative, 
		// but the tetrahedron can be very close to (or touching) the origin
		// Look for farthest plane to decide
		VectorRegister4Float  MaxSignedDistance = VectorZero();
		for (VectorTEPAEntry& Entry : OutEntries)
		{
			const VectorRegister4Float AbsDistance = VectorAbs(Entry.Distance);
			const VectorRegister4Float  AbsMaxSignedDistance = VectorAbs(MaxSignedDistance);
			const VectorRegister4Float AbsDistGTMax = VectorCompareGT(AbsDistance, AbsMaxSignedDistance);
			MaxSignedDistance = VectorSelect(AbsDistGTMax, Entry.Distance, MaxSignedDistance);
		}
		VectorRegister4Float MaxSignedDistanceNeg = VectorCompareGT(VectorZero(), MaxSignedDistance);
		if (::VectorMaskBits(MaxSignedDistanceNeg))
		{
			for (VectorTEPAEntry& Entry : OutEntries)
			{
				Entry.SwapWinding(OutEntries.GetData());
			}
		}
	}

	return bValid;
}


FORCEINLINE_DEBUGGABLE void VectorEPAComputeVisibilityBorder(TEPAWorkingArray<VectorTEPAEntry>& Entries, int32 EntryIdx, const VectorRegister4Float& W, TEPAWorkingArray<FEPAFloodEntry>& OutBorderEdges, TEPAWorkingArray<FEPAFloodEntry>& ToVisitStack)
{
	constexpr VectorRegister4Float Eps = MakeVectorRegisterFloatConstant(1.e-4f, 1.e-4f, 1.e-4f, 1.e-4f);
	{
		VectorTEPAEntry& Entry = Entries[EntryIdx];
		for (int i = 0; i < 3; ++i)
		{
			ToVisitStack.Add({ Entry.AdjFaces[i], Entry.AdjEdges[i] });
		}
	}

	int32 Iteration = 0;
	const int32 MaxIteration = 10000;

	while (ToVisitStack.Num() && Iteration++ < MaxIteration)
	{
		const FEPAFloodEntry FloodEntry = ToVisitStack.Pop(EAllowShrinking::No);
		VectorTEPAEntry& Entry = Entries[FloodEntry.EntryIdx];
		if (!Entry.bObsolete)
		{
			if (::VectorMaskBits(VectorCompareLE(Entry.DistanceToPlane(W), Eps)))
			{
				//W can't see this triangle so mark the edge as a border
				OutBorderEdges.Add(FloodEntry);
			}
			else
			{
				//W can see this triangle so continue flood fill
				Entry.bObsolete = true;	//no longer needed
				const int32 Idx0 = FloodEntry.EdgeIdx;
				const int32 Idx1 = (Idx0 + 1) % 3;
				const int32 Idx2 = (Idx0 + 2) % 3;
				ToVisitStack.Add({ Entry.AdjFaces[Idx1], Entry.AdjEdges[Idx1] });
				ToVisitStack.Add({ Entry.AdjFaces[Idx2], Entry.AdjEdges[Idx2] });
			}
		}
	}

	if (Iteration >= MaxIteration)
	{
		UE_LOGF(LogChaos, Warning, "VectorEPAComputeVisibilityBorder reached max iteration - something is wrong");
	}
}

FORCEINLINE_DEBUGGABLE void VectorComputeEPAResults(const VectorRegister4Float* VertsA, const VectorRegister4Float* VertsB, const VectorTEPAEntry& Entry, VectorRegister4Float& OutPenetration, VectorRegister4Float& OutDir, VectorRegister4Float& OutA, VectorRegister4Float& OutB, EEPAResult& ResultStatus)
{
	//NOTE: We use this function as fallback when robustness breaks. So - do not assume adjacency is valid as these may be new uninitialized traingles that failed

	VectorRegister4Float As[4] = { VertsA[Entry.IdxBuffer[0]], VertsA[Entry.IdxBuffer[1]], VertsA[Entry.IdxBuffer[2]] };
	VectorRegister4Float Bs[4] = { VertsB[Entry.IdxBuffer[0]], VertsB[Entry.IdxBuffer[1]], VertsB[Entry.IdxBuffer[2]] };
	VectorRegister4Float Simplex[4] = { VectorSubtract(As[0], Bs[0]), VectorSubtract(As[1], Bs[1]), VectorSubtract(As[2], Bs[2]) };
	VectorRegister4Float Barycentric;
	int32 NumVerts = 3;
	OutDir = VectorSimplexFindClosestToOrigin(Simplex, NumVerts, Barycentric, As, Bs);

	const VectorRegister4Float DotDir = VectorDot4(OutDir, OutDir);

	if (VectorContainsNaNOrInfinite(DotDir))
	{
		OutPenetration = VectorZero();
		ResultStatus = EEPAResult::NoValidContact;
	}

	OutPenetration = VectorSqrt(DotDir);

	constexpr float EpsFloat = 1e-4f;
	constexpr VectorRegister4Float Eps = MakeVectorRegisterFloatConstant(EpsFloat, EpsFloat, EpsFloat, EpsFloat);

	// @todo(chaos): pass in epsilon? Does it need to match anything in GJK?
	if (::VectorMaskBits(VectorCompareGT(Eps, OutPenetration)))	//if closest point is on the origin (edge case when surface is right on the origin)
	{
		OutDir = Entry.PlaneNormal;	//just fall back on plane normal
		if (::VectorMaskBits(VectorCompareGT(VectorZero(), Entry.Distance)))
		{
			OutPenetration = VectorNegate(OutPenetration); // We are a bit outside of the shape so penetration is negative
		}
	}
	else
	{
		OutDir = VectorDivide(OutDir, OutPenetration);
		if (::VectorMaskBits(VectorCompareGT(VectorZero(), Entry.Distance)))
		{
			//The origin is on the outside, so the direction is reversed
			OutDir = VectorNegate(OutDir);
			OutPenetration = VectorNegate(OutPenetration); // We are a bit outside of the shape so penetration is negative
		}
	}

	OutA = VectorZero();
	OutB = VectorZero();


	VectorRegister4Float Barycentrics[4];
	Barycentrics[0] = VectorSwizzle(Barycentric, 0, 0, 0, 0);
	Barycentrics[1] = VectorSwizzle(Barycentric, 1, 1, 1, 1);
	Barycentrics[2] = VectorSwizzle(Barycentric, 2, 2, 2, 2);
	Barycentrics[3] = VectorSwizzle(Barycentric, 3, 3, 3, 3);

	for (int i = 0; i < NumVerts; ++i)
	{
		OutA = VectorMultiplyAdd(As[i], Barycentrics[i], OutA);
		OutB = VectorMultiplyAdd(Bs[i], Barycentrics[i], OutB);
	}
}

template <typename TSupportA, typename TSupportB>
EEPAResult VectorEPA(TArray<VectorRegister4Float>& VertsABuffer, TArray<VectorRegister4Float>& VertsBBuffer, const TSupportA& SupportA, const TSupportB& SupportB, VectorRegister4Float& OutPenetration, VectorRegister4Float& OutDir, VectorRegister4Float& WitnessA, VectorRegister4Float& WitnessB)
{
	const VectorRegister4Float OriginInsideEps = VectorZero();

	TEPAWorkingArray<VectorTEPAEntry> Entries;

	if(!VectorInitializeEPA(VertsABuffer, VertsBBuffer, SupportA, SupportB, Entries, OutDir))
	{
		//either degenerate or a touching hit. Either way return penetration 0
		OutPenetration = VectorZero();
		WitnessA = VectorZero();
		WitnessB = VectorZero();
		return EEPAResult::BadInitialSimplex;
	}

#if DEBUG_EPA
	TArray<TVec3<FRealSingle>> VertsWBuffer;
	for (int32 Idx = 0; Idx < 4; ++Idx)
	{
		VertsWBuffer.Add(VectorMinkowskiVert(VertsABuffer.GetData(), VertsBBuffer.GetData(), Idx));
	}
#endif
	
	TEPAWorkingArray<int32> Queue;
	for(int32 Idx = 0; Idx < Entries.Num(); ++Idx)
	{
		const bool bIsZeroGTDist = static_cast<bool>(::VectorMaskBits(VectorCompareGE(VectorZero(), Entries[Idx].Distance)));
		//ensure(Entries[Idx].Distance > -Eps);
		// Entries[Idx].Distance <= 0.0f is true if the origin is a bit out of the polytope (we need to support this case for robustness)
		if(bIsZeroGTDist || Entries[Idx].IsOriginProjectedInside(VertsABuffer.GetData(), VertsBBuffer.GetData(), OriginInsideEps))
		{
			Queue.Add(Idx);
		}
	}

	TEPAWorkingArray<FEPAFloodEntry> VisibilityBorder;
	TEPAWorkingArray<FEPAFloodEntry> VisibilityBorderToVisitStack;

	VectorTEPAEntry LastEntry = Queue.Num() > 0 ? Entries[Queue.Last()] : Entries[0];

	constexpr VectorRegister4Float MaxLimit = MakeVectorRegisterFloatConstant(TNumericLimits<FRealSingle>::Max(), TNumericLimits<FRealSingle>::Max(), TNumericLimits<FRealSingle>::Max(), TNumericLimits<FRealSingle>::Max());
	constexpr VectorRegister4Float MinLimit = MakeVectorRegisterFloatConstant(TNumericLimits<FRealSingle>::Lowest(), TNumericLimits<FRealSingle>::Lowest(), TNumericLimits<FRealSingle>::Lowest(), TNumericLimits<FRealSingle>::Lowest());

	VectorRegister4Float UpperBound = MaxLimit;
	VectorRegister4Float LowerBound = MinLimit;
	bool bQueueDirty = true;
	int32 Iteration = 0;
	int32 constexpr MaxIterations = 128;
	EEPAResult ResultStatus = EEPAResult::MaxIterations;
	while (Queue.Num() && (Iteration++ < MaxIterations))
	{
		if (bQueueDirty)
		{
			// Avoiding UE's Sort here because it has a call to FMath::Loge. The std version calls insertion sort when possible
			std::sort(Queue.GetData(), Queue.GetData() + Queue.Num(), [&Entries](const int32 L, const int32 R) { return Entries[L] > Entries[R]; });
			bQueueDirty = false;
		}

		int32 EntryIdx = Queue.Pop(EAllowShrinking::No);
		VectorTEPAEntry& Entry = Entries[EntryIdx];
		//bool bBadFace = Entry.IsOriginProjectedInside(VertsABuffer.GetData(), VertsBBuffer.GetData());
		{
			//UE_LOGF(LogChaos, Warning, "%d BestW:%f, Distance:%f, bObsolete:%d, InTriangle:%d",
			//	Iteration, BestW.Size(), Entry.Distance, Entry.bObsolete, bBadFace);
		}
		if (Entry.bObsolete)
		{
			// @todo(chaos): should this count as an iteration? Currently it does...
			continue;
		}

		if (::VectorMaskBits(VectorCompareGT(Entry.Distance, UpperBound)))
		{
			ResultStatus = EEPAResult::Ok;
			break;
		}

		const VectorRegister4Float ASupport = SupportA(Entry.PlaneNormal);
		const VectorRegister4Float BSupport = SupportB(VectorNegate(Entry.PlaneNormal));
		const VectorRegister4Float W = VectorSubtract(ASupport, BSupport);
		const VectorRegister4Float DistanceToSupportPlane = VectorDot3(Entry.PlaneNormal, W);

		//Remember the entry that gave us the lowest upper bound and use it in case we have to terminate early
		//This can result in very deep planes. Ideally we'd just use the plane formed at W, but it's not clear how you get back points in A, B for that
		//BestEntry = Entry;
		UpperBound = VectorMin(DistanceToSupportPlane, UpperBound);
		LowerBound = Entry.Distance;

		// Add one absolute epsilon to be more robust with small values, fixed some issue with small UpperBound
		constexpr VectorRegister4Float SmallFloatEps = MakeVectorRegisterFloatConstant(1.e-5f, 1.e-5f, 1.e-5f, 1.e-5f);
		// Use a relative epsilon to have a robust check in the main case
		constexpr VectorRegister4Float OnePlusRelativeEps = MakeVectorRegisterFloatConstant(1.01f, 1.01f, 1.01f, 1.01f);
		// It's possible the origin is not contained by the CSO, probably because of numerical error. 
		// In this case the upper bound will be negative, at which point we should just exit. 
		const VectorRegister4Float UpperBoundTolerance = VectorAdd(VectorMultiply(OnePlusRelativeEps, VectorAbs(LowerBound)), SmallFloatEps);
		const VectorRegister4Float UpperBoundLEUpperBoundTolerance = VectorCompareLE(UpperBound, UpperBoundTolerance);
		if (::VectorMaskBits(UpperBoundLEUpperBoundTolerance))
		{
			ResultStatus = EEPAResult::Ok;
			LastEntry = Entry;
			break;
		}

		const VectorRegister4Float LowGTUp = VectorCompareGT(LowerBound, UpperBound);
		if (::VectorMaskBits(LowGTUp))
		{
			//we cannot get any better than what we saw, so just return previous face
			ResultStatus = EEPAResult::Ok;
			break;
		}

		LastEntry = Entry;

		const int32 NewVertIdx = VertsABuffer.Add(ASupport);
		VertsBBuffer.Add(BSupport);

#if DEBUG_EPA
		VertsWBuffer.Add(VectorMinkowskiVert(VertsABuffer.GetData(), VertsBBuffer.GetData(), NewVertIdx));
#endif

		Entry.bObsolete = true;
		VisibilityBorder.Reset();
		VisibilityBorderToVisitStack.Reset();
		VectorEPAComputeVisibilityBorder(Entries, EntryIdx, W, VisibilityBorder, VisibilityBorderToVisitStack);
		const int32 NumBorderEdges = VisibilityBorder.Num();
		const int32 FirstIdxInBatch = Entries.Num();
		int32 NewIdx = FirstIdxInBatch;
		Entries.AddUninitialized(NumBorderEdges);
		if (NumBorderEdges >= 3)
		{
			bool bTerminate = false;
			for (int32 VisibilityIdx = 0; VisibilityIdx < NumBorderEdges; ++VisibilityIdx)
			{
				//create new entries and update adjacencies
				const FEPAFloodEntry& BorderInfo = VisibilityBorder[VisibilityIdx];
				VectorTEPAEntry& NewEntry = Entries[NewIdx];
				const int32 BorderEntryIdx = BorderInfo.EntryIdx;
				VectorTEPAEntry& BorderEntry = Entries[BorderEntryIdx];
				const int32 BorderEdgeIdx0 = BorderInfo.EdgeIdx;
				const int32 BorderEdgeIdx1 = (BorderEdgeIdx0 + 1) % 3;
				const int32 NextEntryIdx = (VisibilityIdx + 1) < VisibilityBorder.Num() ? NewIdx + 1 : FirstIdxInBatch;
				const int32 PrevEntryIdx = NewIdx > FirstIdxInBatch ? NewIdx - 1 : FirstIdxInBatch + NumBorderEdges - 1;
				const bool bValidTri = NewEntry.Initialize(VertsABuffer.GetData(), VertsBBuffer.GetData(), BorderEntry.IdxBuffer[BorderEdgeIdx1], BorderEntry.IdxBuffer[BorderEdgeIdx0], NewVertIdx,
					{ BorderEntryIdx, PrevEntryIdx, NextEntryIdx },
					{ BorderEdgeIdx0, 2, 1 });
				BorderEntry.AdjFaces[BorderEdgeIdx0] = NewIdx;
				BorderEntry.AdjEdges[BorderEdgeIdx0] = 0;
				
				if (!bValidTri)
				{
					//couldn't properly expand polytope, so just stop
					ResultStatus = EEPAResult::Degenerate;
					bTerminate = true;
					break;
				}

				// Due to numerical inaccuracies NewEntry.Distance >= LowerBound may be false!
				// However these Entries still have good normals, and needs to be included to prevent
				// this exiting with very deep penetration results
				
				if (bValidTri && ::VectorMaskBits(VectorCompareGE(UpperBound, NewEntry.Distance)))
				{
					// NewEntry.Distance <= 0.0f is if the origin is a bit out of the polytope
					if (::VectorMaskBits(VectorCompareGE(VectorZero(), NewEntry.Distance))|| NewEntry.IsOriginProjectedInside(VertsABuffer.GetData(), VertsBBuffer.GetData(), OriginInsideEps))
					{
						Queue.Add(NewIdx);
						bQueueDirty = true;
					}
				}

				++NewIdx;
			}

			if (bTerminate)
			{
				break;
			}
		}
		else
		{
			//couldn't properly expand polytope, just stop now
			ResultStatus = EEPAResult::Degenerate;
			break;
		}
	}

	VectorComputeEPAResults(VertsABuffer.GetData(), VertsBBuffer.GetData(), LastEntry, OutPenetration, OutDir, WitnessA, WitnessB, ResultStatus);
	
	return ResultStatus;
}
}