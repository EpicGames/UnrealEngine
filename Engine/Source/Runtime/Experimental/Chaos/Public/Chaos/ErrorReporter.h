// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "ChaosLog.h"
#include "Logging/LogMacros.h"

namespace Chaos
{
class FErrorReporter
{
public:
	FErrorReporter(FString ErrorPrefix = "")
	: bEncountedErrors(false)
	, bUnhandledErrors(false)
	, Prefix(ErrorPrefix)
	{
	}

	void HandleLatestError()
	{
		bUnhandledErrors = false;
	}

	bool ContainsUnhandledError() const
	{
		return bUnhandledErrors;
	}

	void ReportLog(const TCHAR* ErrorMsg)
	{
		if(Prefix != "")
		{
			UE_LOGF(LogChaos, Log, "ErrorReporter (%ls): %ls", *Prefix, ErrorMsg);
		}
		else
		{
			UE_LOGF(LogChaos, Log, "ErrorReporter: %ls", ErrorMsg);
		}
	}

	void ReportWarning(const TCHAR* ErrorMsg)
	{
		if (Prefix != "")
		{
			UE_LOGF(LogChaos, Warning, "ErrorReporter (%ls): %ls", *Prefix, ErrorMsg);
		}
		else
		{
			UE_LOGF(LogChaos, Warning, "ErrorReporter: %ls", ErrorMsg);
		}
	}

	void ReportError(const TCHAR* ErrorMsg)
	{
		ReportWarning(ErrorMsg);
		bEncountedErrors = true;
		bUnhandledErrors = true;
	}

	bool EncounteredAnyErrors() const
	{
		return bEncountedErrors;
	}

	void SetPrefix(FString NewPrefix)
	{
		Prefix = NewPrefix;
	}

	FString GetPrefix()
	{
		return Prefix;
	}

private:
	bool bEncountedErrors;
	bool bUnhandledErrors;
	FString Prefix;
};
}
