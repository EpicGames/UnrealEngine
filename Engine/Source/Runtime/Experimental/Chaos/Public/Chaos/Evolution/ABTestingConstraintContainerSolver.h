// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Evolution/SolverBodyContainer.h"
#include "Chaos/Evolution/SolverConstraintContainer.h"
#include <functional>

#ifndef CHAOS_ABTEST_CONSTRAINTSOLVER_ENABLED
#define CHAOS_ABTEST_CONSTRAINTSOLVER_ENABLED (!UE_BUILD_TEST && !UE_BUILD_SHIPPING)
#endif


namespace Chaos
{
	class FConstraintHandleHolder;
	class FSolverBodyContainer;

	namespace Private
	{
#if CHAOS_ABTEST_CONSTRAINTSOLVER_ENABLED

		/**
		* Used for testing a new solver that is supposed to produce the same output as anotehr solver.
		* E.g., for testing the Simd version of a solver against the non-simd version.
		* 
		* The first solver passed in is assumed to be the reference - its results will be used to generate the outputs.
		* The second solver results will be compared and discarded.
		* 
		* @todo(chaos): this isn't good enough on its own. Really we need to AB Test the whole IslandGroupSolver
		* but with a different set of constraint container solvers in each. The way it is now will work fine
		* as long as we only have one constraint type in the island group which will do for now.
		*/
		template<typename T1, typename T2>
		class TABTestingConstraintContainerSolver : public FConstraintContainerSolver
		{
		public:
			enum class ESolverPhase
			{
				PreApplyPositionConstraints,
				PostApplyPositionConstraints,
				PreApplyVelocityConstraints,
				PostApplyVelocityConstraints,
				PreApplyProjectionConstraints,
				PostApplyProjectionConstraints,
			};

			using FContainerSolverTypeA = T1;
			using FContainerSolverTypeB = T2;
			using FABTestFunctor = std::function<void(
				const ESolverPhase Phase, 
				const FContainerSolverTypeA& SolverA, 
				const FContainerSolverTypeB& SolverB, 
				const FSolverBodyContainer& SolverBodyContainerA, 
				const FSolverBodyContainer& SolverBodyContainerB)>;

			TABTestingConstraintContainerSolver(
				TUniquePtr<FContainerSolverTypeA>&& InSolverA, 
				TUniquePtr<FContainerSolverTypeB>&& InSolverB, 
				const int32 InPriority,
				const FABTestFunctor& InABTestFunctor)
				: FConstraintContainerSolver(InPriority)
				, SolverA(MoveTemp(InSolverA))
				, SolverB(MoveTemp(InSolverB))
				, SolverBodyContainerA(nullptr)
				, SolverBodyContainerB()
				, ABTestFunctor(InABTestFunctor)
			{
			}

			virtual int32 GetNumConstraints() const override final
			{
				return SolverA->GetNumConstraints();
			}

			virtual void AddConstraints() override final
			{
				SolverA->AddConstraints();
				SolverB->AddConstraints();
			}

			virtual void AddConstraints(const TArrayView<Private::FPBDIslandConstraint*>& Constraints) override final
			{
				SolverA->AddConstraints(Constraints);
				SolverB->AddConstraints(Constraints);
			}

			virtual void AddBodies(FSolverBodyContainer& SolverBodyContainer) override final
			{
				SolverBodyContainerA = &SolverBodyContainer;

				SolverA->AddBodies(*SolverBodyContainerA);
			}

			virtual void GatherInput(const FReal Dt) override final
			{
				SolverA->GatherInput(Dt);
				// Solver B will ge gathered in PreApply
			}

			virtual void GatherInput(const FReal Dt, const int32 BeginIndex, const int32 EndIndex) override final
			{
				SolverA->GatherInput(Dt, BeginIndex, EndIndex);
				// Solver B will ge gathered in PreApply
			}

			virtual void ScatterOutput(const FReal Dt) override final
			{
				SolverA->ScatterOutput(Dt);
				SolverBodyContainerA = nullptr;
				// Do not scatter output from B!
			}

			virtual void ScatterOutput(const FReal Dt, const int32 BeginIndex, const int32 EndIndex) override final
			{
				SolverA->ScatterOutput(Dt, BeginIndex, EndIndex);
				SolverBodyContainerA = nullptr;
				// Do not scatter output from B!
			}

			virtual void PreApplyPositionConstraints(const FReal Dt) override final
			{
				// Copy all the solver bodies for use by the second solver
				SolverBodyContainerA->CopyTo(SolverBodyContainerB);

				// Bind SolverB to the copied bodies
				SolverB->AddBodies(SolverBodyContainerB);

				// Gather the second solver's constraints
				SolverB->GatherInput(Dt);

				SolverA->PreApplyPositionConstraints(Dt);
				SolverB->PreApplyPositionConstraints(Dt);

				CallABTestFunctor(ESolverPhase::PreApplyPositionConstraints);
			}

			virtual void ApplyPositionConstraints(const FReal Dt, const int32 It, const int32 NumIts) override final
			{
				SolverA->ApplyPositionConstraints(Dt, It, NumIts);
				SolverB->ApplyPositionConstraints(Dt, It, NumIts);

				CallABTestFunctor(ESolverPhase::PostApplyPositionConstraints);
			}

			virtual void ApplyPositionConstraints(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex) override final
			{
				SolverA->ApplyPositionConstraints(Dt, It, NumIts, BeginIndex, EndIndex);
				SolverB->ApplyPositionConstraints(Dt, It, NumIts, BeginIndex, EndIndex);

				CallABTestFunctor(ESolverPhase::PostApplyPositionConstraints);
			}

			virtual void PreApplyVelocityConstraints(const FReal Dt) override final
			{
				SolverBodyContainerB.SetImplicitVelocities(Dt);

				SolverA->PreApplyVelocityConstraints(Dt);
				SolverB->PreApplyVelocityConstraints(Dt);

				CallABTestFunctor(ESolverPhase::PreApplyVelocityConstraints);
			}

			virtual void ApplyVelocityConstraints(const FReal Dt, const int32 It, const int32 NumIts) override final
			{
				SolverA->ApplyVelocityConstraints(Dt, It, NumIts);
				SolverB->ApplyVelocityConstraints(Dt, It, NumIts);

				CallABTestFunctor(ESolverPhase::PostApplyVelocityConstraints);
			}

			virtual void ApplyVelocityConstraints(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex) override final
			{
				SolverA->ApplyVelocityConstraints(Dt, It, NumIts, BeginIndex, EndIndex);
				SolverB->ApplyVelocityConstraints(Dt, It, NumIts, BeginIndex, EndIndex);

				CallABTestFunctor(ESolverPhase::PostApplyVelocityConstraints);
			}

			virtual void PreApplyProjectionConstraints(const FReal Dt) override final
			{
				SolverBodyContainerB.ApplyCorrections();

				SolverA->PreApplyProjectionConstraints(Dt);
				SolverB->PreApplyProjectionConstraints(Dt);

				CallABTestFunctor(ESolverPhase::PreApplyProjectionConstraints);
			}

			virtual void ApplyProjectionConstraints(const FReal Dt, const int32 It, const int32 NumIts) override final
			{
				SolverA->ApplyProjectionConstraints(Dt, It, NumIts);
				SolverB->ApplyProjectionConstraints(Dt, It, NumIts);

				CallABTestFunctor(ESolverPhase::PostApplyProjectionConstraints);
			}

			virtual void ApplyProjectionConstraints(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex) override final
			{
				SolverA->ApplyProjectionConstraints(Dt, It, NumIts, BeginIndex, EndIndex);
				SolverB->ApplyProjectionConstraints(Dt, It, NumIts, BeginIndex, EndIndex);

				CallABTestFunctor(ESolverPhase::PostApplyProjectionConstraints);
			}

		protected:
			void CallABTestFunctor(const ESolverPhase Phase)
			{
				if (ABTestFunctor != nullptr)
				{
					ABTestFunctor(Phase, *SolverA, *SolverB, *SolverBodyContainerA, SolverBodyContainerB);
				}
			}

			TUniquePtr<FContainerSolverTypeA> SolverA;
			TUniquePtr<FContainerSolverTypeB> SolverB;
			FSolverBodyContainer* SolverBodyContainerA;
			FSolverBodyContainer SolverBodyContainerB;
			FABTestFunctor ABTestFunctor;

		private:
			virtual void ResetImpl(const int32 MaxConstraints) override final
			{
				SolverA->Reset(MaxConstraints);
				SolverB->Reset(MaxConstraints);
				SolverBodyContainerA = nullptr;
			}

			virtual bool SupportsPartitioning() const override final { return false; }

			virtual bool SortConstraintsByPartitionIndex(const TArray<int32>& PartitionIndices, const TArray<int32>& PrefixSum) override final
			{
				// @todo(Chaos): Do we want to implement a parallelized solver option here?
				return false;
			}

			virtual void GetAllConstrainedParticles(TArray<const FGeometryParticleHandle*>& ParticleHandles) const override final
			{ 
				// @todo(Chaos): Do we want to implement a parallelized solver option here?
				ParticleHandles.Reset();
			}
		};

#endif // CHAOS_ABTEST_CONSTRAINTSOLVER_ENABLED

	}	// namespace Private
}	// namespace Chaos
