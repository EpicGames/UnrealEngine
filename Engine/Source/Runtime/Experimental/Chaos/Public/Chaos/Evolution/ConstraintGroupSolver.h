// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/Evolution/IterationSettings.h"
#include "Chaos/Evolution/SolverBodyContainer.h"
#include "Containers/Array.h"
#include "Tasks/Task.h"

namespace Chaos
{
	class FConstraintContainerSolver;
	class FPBDConstraintContainer;

	namespace Private
	{
		/**
		 * All the data required to solver a set of constraints.
		 * All constraints in the groups are solved in sequence in a single thread. We can create one of these for 
		 * each Island (or group of Islands, or subset of a color of constraints) for parallelism.
		*/
		class FPBDConstraintGroupSolver
		{
		public:
			UE_NONCOPYABLE(FPBDConstraintGroupSolver);

			CHAOS_API FPBDConstraintGroupSolver();
			CHAOS_API virtual ~FPBDConstraintGroupSolver();

			/**
			 * Get the iterations settings for this group of islands
			*/
			FIterationSettings GetIterationSettings() const
			{
				return Iterations;
			}

			/**
			 * Set the iterations settings for this group of islands
			*/
			virtual void SetIterationSettings(const FIterationSettings& InIterations)
			{
				Iterations = InIterations;
			}

			/**
			 * Get the number of solver bodies we have. This is all the bodies referenced by all constraints and is only non-zero
			 * after calling AddConstraintsAndBodies(). NOTE: it may be less than the number of particles in the IslandGroup 
			 * (@see FPBDISlandGroup) when there are no constraints in some islands.
			*/
			inline int32 GetNumSolverBodies() const
			{
				return SolverBodyContainer.Num();
			}

			/**
			 * Get the number of constraint solver we have, after AddConstraintsAndBodies() has been called.
			*/
			inline int32 GetNumSolverConstraints() const
			{
				return TotalNumConstraints;
			}

			/**
			 * Attach a constraint solver to the specified ContainerId. This must be for the same constraint type as the container with that Id.
			*/
			CHAOS_API void SetConstraintSolver(const int32 ContainerId, TUniquePtr<FConstraintContainerSolver>&& Solver);

			/**
			 * Set the solver priority of the specified constraint type
			*/
			CHAOS_API void SetConstraintSolverPriority(const int32 ContainerId, const int32 Priority);

			/**
			 * Reset all state - called once per tick
			*/
			CHAOS_API void Reset();

			/**
			 * Set up the constraints solvers and body containers with pointers to their constraint and particles, but do not collect any data.
			*/
			CHAOS_API void AddConstraintsAndBodies();

			/**
			 * Collect all the data for all solver bodies from their respective particles.
			*/
			CHAOS_API void GatherBodies(const FReal Dt);

			/**
			 * Collect all the data for the specified range of solver bodies from their respective particles. Will be called from multiple
			 * threads with different non-overlapping indices.
			*/
			CHAOS_API void GatherBodies(const FReal Dt, const int32 BeginBodyIndex, const int32 EndBodyIndex);

			/**
			 * Collect all the data for all constraint solvers from their respective constraints.
			*/
			CHAOS_API void GatherConstraints(const FReal Dt);

			/**
			 * Collect all the data for the specified range of constraint solvers from their respective constraints. Will be called from multiple
			 * threads with different non-overlapping indices.
			*/
			CHAOS_API void GatherConstraints(const FReal Dt, const int32 BeginConstraintIndex, const int32 EndConstraintIndex);

			/**
			 * For additional processing after gathering all the data.
			 * Calls PreApplyConstraints on each container solver.
			 */
			CHAOS_API void PreApplyPositionConstraints(const FReal Dt);
			CHAOS_API void PreApplyVelocityConstraints(const FReal Dt);
			CHAOS_API void PreApplyProjectionConstraints(const FReal Dt);

			/**
			 * Apply positional constraints, and set the body velocities
			*/
			CHAOS_API void ApplyPositionConstraints(const FReal Dt);

			/**
			 * Apply any velocity constraints, and update the body velocities
			*/
			CHAOS_API void ApplyVelocityConstraints(const FReal Dt);

			/**
			 * Apply projection to attempt to fix up any errors left over from the Position and Velocity saolver phases
			*/
			CHAOS_API void ApplyProjectionConstraints(const FReal Dt);

			/**
			 * Calls the private pre-apply and apply constraints functions in the following order:
			 *  PreApplyPositionConstraintsParallelTasks
			 *  ApplyPositionConstraintsParallelTasks
			 * 
			 *  PreApplyVelocityConstraintsParallelTasks
			 *  ApplyVelocityConstraintsParallelTasks
			 * 
			 *  PreApplyProjectionConstraintsParallelTasks
			 *  ApplyProjectionConstraintsParallelTasks
			 * 
			 * This is equivalent to calling the public pre-apply and apply constraints functions,
			 * however, it parallelizes the constraint solve by scheduling and executing tasks
			 * provided their respective prerequisite tasks.
			 * 
			 * The function returns a TArray<UE::Tasks::FTask> of the tasks to be executed last.
			 * The caller of this function must do one of the following:
			 *   1) if there will be more tasks launched, use the returned tasks as a prerequisites for the next task
			 *   2) if these were the last tasks and this is NOT called from within task, call UE::Tasks::Wait.
			 *   3) if these were the last tasks and this IS called from within task, call UE::Tasks::AddNested.
			*/
			CHAOS_API TArray<UE::Tasks::FTask> ApplyConstraintsParallelTasks(const FReal Dt);

			/**
			 * Push results from all solver bodies back to their respective particles.
			*/
			CHAOS_API void ScatterBodies(const FReal Dt);

			/**
			 * Push results from the specified range of solver bodies back to their respective particles. Will be called from multiple
			 * threads with different non-overlapping indices.
			*/
			CHAOS_API void ScatterBodies(const FReal Dt, const int32 BeginBodyIndex, const int32 EndBodyIndex);

			/**
			 * Push results from all constraint solvers back to their respective constraints.
			*/
			CHAOS_API void ScatterConstraints(const FReal Dt);

			/**
			 * Push results from the specified range of constraint solvers back to their respective constraints.Will be called from multiple
			 * threads with different non-overlapping indices. The range covers all constraint types.
			*/
			CHAOS_API void ScatterConstraints(const FReal Dt, const int32 BeginConstraintIndex, const int32 EndConstraintIndex);

			/**
			 * Decides whether the constraint solver is called in parallel batches
			*/
			CHAOS_API const bool SolveInParallel() const;

			/**
			 * Returns the number of serial constraint batches for the given ContainerId.
			 * Returns 0 if partitioning is disabled or the container doesn't exist in this group.
			*/
			UE_INTERNAL int32 GetNumSerialConstraintBatches(const int32 ContainerId) const;

		protected:

			// Loop over the range of constraints and apply the lambda. The range is treats constraints of all types as a contiguous list
			// and the function loops over constraint types and maps the range into the solver ranges for each type.
			template<typename LambdaType>
			void ApplyToConstraintRange(const int32 BeginConstraintIndex, const int32 EndConstraintIndex, const LambdaType& Lambda);

			// Sort the solvers based on Level and other criteria for more stable solving
			CHAOS_API void SortSolverContainers();

			// Allow a derived class to perform per-tick reset
			virtual void ResetImpl() {}

			// Allow a derived class to add data based on the number of constraint types we need to support
			virtual void SetConstraintSolverImpl(const int32 ContainerId) {}

			// Allow a derived class to add constraints solvers
			virtual void AddConstraintsImpl() {}

			// Allow a derived class to populate some of the SolverBody data (Level and Color)
			virtual void GatherBodiesImpl(const FReal Dt, const int32 BeginBodyIndex, const int32 EndBodyIndex) {}

			// All the bodies used by all the constraints solved by this group
			FSolverBodyContainer SolverBodyContainer;

			// A solver for each constraint type in the group
			TArray<TUniquePtr<FConstraintContainerSolver>> ConstraintContainerSolvers;
			int32 TotalNumConstraints;

			// Sorted by priority for use in solve methods
			TArray<FConstraintContainerSolver*> PrioritizedConstraintContainerSolvers;

			// Parallelized solve only
			int32 MaxSerialBatches;
			TArray<TArray<UE::Tasks::FTask>> ConstraintSolvePrerequisites;

			FIterationSettings Iterations;

		private:
			template<typename Lambda>
			void DispatchUpdateSolverBodies(Lambda UpdateSolverBodies, const FReal Dt);

			template<typename Lambda>
			void DispatchApplyConstraints(Lambda ApplyConstraints, const FReal Dt, const int32 It, const int32 NumIts);

			// Parallelized version of the public pre-apply and apply functions
			// These functions are private because they must only be called by ApplyConstraintsParallelTasks.
			void PreApplyPositionConstraintsParallelTasks(const FReal Dt);
			void PreApplyVelocityConstraintsParallelTasks(const FReal Dt);
			void PreApplyProjectionConstraintsParallelTasks(const FReal Dt);
			void ApplyPositionConstraintsParallelTasks(const FReal Dt);
			void ApplyVelocityConstraintsParallelTasks(const FReal Dt);
			void ApplyProjectionConstraintsParallelTasks(const FReal Dt);
		};


		//
		//
		//
		//
		//

		/**
		 * A Constraint Solver that solves all constraints in the scene in sequence. Used by RBAN and tests, as
		 * opposed to the FPBDIslandConstraintGroupSolver used by the main scene and has a constraint graph to batch constraints
		 * into non-interacting islands for parallelization.
		 * 
		 * There will only be one (or zero) FPBDSceneConstraintGroupSolver per simulation world.
		*/
		class FPBDSceneConstraintGroupSolver : public FPBDConstraintGroupSolver
		{
		public:
			FPBDSceneConstraintGroupSolver(const FIterationSettings& InIterations)
			{
				Iterations = InIterations;
			}

		protected:
			// The Scene Group Solver adds all constraints from all containers
			CHAOS_API virtual void AddConstraintsImpl() override final;
		};

	} // namespace Private


	//UE_DEPRECATED(5.2, "FPBDConstraintGroupSolver is for internal use only")
	//using FPBDConstraintGroupSolver = Private::FPBDConstraintGroupSolver;


	//UE_DEPRECATED(5.2, "FPBDSceneConstraintGroupSolver is for internal use only")
	//using FPBDSceneConstraintGroupSolver = Private::FPBDSceneConstraintGroupSolver;

}	// namespace Chaos
