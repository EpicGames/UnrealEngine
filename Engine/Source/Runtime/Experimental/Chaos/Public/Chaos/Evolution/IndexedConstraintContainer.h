// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/ConstraintHandle.h"
#include "Chaos/Evolution/SolverConstraintContainer.h"
#include "Chaos/Island/IslandManager.h"
#include "Chaos/PBDConstraintContainer.h"

namespace Chaos
{
	/**
	 * Base class for handles to constraints in an index-based container (e.g., FJointConstraints)
	 */
	class FIndexedConstraintHandle : public FConstraintHandle
	{
	public:
		using FGeometryParticleHandle = TGeometryParticleHandle<FReal, 3>;

		FIndexedConstraintHandle()
			: FConstraintHandle()
			, ConstraintIndex(INDEX_NONE)
		{
		}

		FIndexedConstraintHandle(FPBDConstraintContainer* InContainer, int32 InConstraintIndex)
			: FConstraintHandle(InContainer)
			, ConstraintIndex(InConstraintIndex)
		{
		}

		virtual ~FIndexedConstraintHandle() = default;

		virtual bool IsValid() const override
		{
			return (ConstraintIndex != INDEX_NONE) && FConstraintHandle::IsValid();
		}

		int32 GetConstraintIndex() const
		{
			return ConstraintIndex;
		}

		void SetConstraintIndex(const int32 InConstraintIndex)
		{
			ConstraintIndex = InConstraintIndex;
		}

		static const FConstraintHandleTypeID& StaticType()
		{
			static FConstraintHandleTypeID STypeID(TEXT("FIndexedConstraintHandle"), &FConstraintHandle::StaticType());
			return STypeID;
		}

	protected:
		friend class FPBDIndexedConstraintContainer;

		int32 ConstraintIndex;
	};


	/**
	 * Utility base class for FIndexedConstraintHandles
	 */
	template<typename T_CONTAINER>
	class TIndexedContainerConstraintHandle : public FIndexedConstraintHandle
	{
	public:
		using Base = FIndexedConstraintHandle;
		using FGeometryParticleHandle = typename Base::FGeometryParticleHandle;
		using FConstraintContainer = T_CONTAINER;

		TIndexedContainerConstraintHandle()
			: FIndexedConstraintHandle()
		{
		}

		TIndexedContainerConstraintHandle(FConstraintContainer* InConstraintContainer, int32 InConstraintIndex)
			: FIndexedConstraintHandle(InConstraintContainer, InConstraintIndex)
		{
		}

		inline virtual void SetEnabled(bool bInEnabled) override
		{
			if (ConcreteContainer() != nullptr)
			{
				ConcreteContainer()->SetConstraintEnabled(ConstraintIndex, bInEnabled);
			}
		}

		inline virtual bool IsEnabled() const override
		{
			if (ConcreteContainer() != nullptr)
			{
				return ConcreteContainer()->IsConstraintEnabled(ConstraintIndex);
			}
			return false;
		}

		// @todo(chaos): Make this a virtual on FConstraintContainer and move to base class
		void RemoveConstraint()
		{
			ConcreteContainer()->RemoveConstraint(ConstraintIndex);
		}

	protected:
		FConstraintContainer* ConcreteContainer()
		{
			return static_cast<FConstraintContainer*>(ConstraintContainer);
		}

		const FConstraintContainer* ConcreteContainer() const
		{
			return static_cast<const FConstraintContainer*>(ConstraintContainer);
		}

		using Base::ConstraintIndex;
	};

	/**
	 * A solver container for use by constraint containers that implement the solve methods directly and hold all
	 * their constraints in arrays and (Joints, Suspension, ...). This Solver just calls into the Container with
	 * the list of constraint indices to solve. There will be one of these created for each constraint solver task,
	 * as determined by the constraint graph.
	*/
	template<typename ConstraintContainerType>
	class TIndexedConstraintContainerSolver : public FConstraintContainerSolver
	{
	public:
		using FConstraintContainerType = ConstraintContainerType;
		using FConstraintHandleType = typename FConstraintContainerType::FConstraintContainerHandle;

		TIndexedConstraintContainerSolver(FConstraintContainerType& InConstraintContainer, const int32 InPriority)
			: FConstraintContainerSolver(InPriority)
			, ConstraintContainer(InConstraintContainer)
		{
		}

		virtual int32 GetNumConstraints() const override final
		{
			return ConstraintIndices.Num();
		}

		virtual void AddConstraints() override final
		{
			// This solver container is for use with the constraint graph. It will not call this function.
			ensure(false);
		}

		virtual void AddConstraints(const TArrayView<Private::FPBDIslandConstraint*>& IslandConstraints) override final
		{
			for (const Private::FPBDIslandConstraint* IslandConstraint : IslandConstraints)
			{
				// Filter out sleeping constraints in any partially sleeping island
				if (!IslandConstraint->IsSleeping())
				{
					FConstraintHandle* Constraint = IslandConstraint->GetConstraint();

					// We will only ever be given constraints from our container (asserts in non-shipping)
					const FIndexedConstraintHandle* IndexedConstraintHandle = Constraint->AsUnsafe<FIndexedConstraintHandle>();

					int32 ConstraintIndex = IndexedConstraintHandle->GetConstraintIndex();

					ConstraintIndices.Add(ConstraintIndex);
				}
			}
		}

		virtual void AddBodies(FSolverBodyContainer& SolverBodyContainer) override final
		{
			ConstraintContainer.AddBodies(ConstraintIndices, SolverBodyContainer);
		}

		virtual void GatherInput(const FReal Dt) override final
		{
			ConstraintContainer.GatherInput(ConstraintIndices, Dt);
		}

		virtual void GatherInput(const FReal Dt, const int32 BeginIndex, const int32 EndIndex) override final
		{
			check(BeginIndex >= 0);
			check(EndIndex <= GetNumConstraints());
			
			const int32 RangeSize = EndIndex - BeginIndex;
			if (RangeSize > 0)
			{
				TArrayView<int32> RangeIndices = MakeArrayView(&ConstraintIndices[BeginIndex], RangeSize);
				ConstraintContainer.GatherInput(RangeIndices, Dt);
			}
		}

		virtual void ScatterOutput(const FReal Dt) override final
		{
			ConstraintContainer.ScatterOutput(ConstraintIndices, Dt);
		}

		virtual void ScatterOutput(const FReal Dt, const int32 BeginIndex, const int32 EndIndex) override final
		{
			check(BeginIndex >= 0);
			check(EndIndex <= GetNumConstraints());

			const int32 RangeSize = EndIndex - BeginIndex;
			if (RangeSize > 0)
			{
				TArrayView<int32> RangeIndices = MakeArrayView(&ConstraintIndices[BeginIndex], RangeSize);
				ConstraintContainer.ScatterOutput(RangeIndices, Dt);
			}
		}

		virtual void ApplyPositionConstraints(const FReal Dt, const int32 It, const int32 NumIts) override final
		{
			ConstraintContainer.ApplyPositionConstraints(ConstraintIndices, Dt, It, NumIts);
		}

		virtual void ApplyPositionConstraints(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex) override final
		{
			check(BeginIndex >= 0);
			check(EndIndex <= GetNumConstraints());

			const int32 RangeSize = EndIndex - BeginIndex;
			if (RangeSize > 0)
			{
				TArrayView<int32> RangeIndices = MakeArrayView(&ConstraintIndices[BeginIndex], RangeSize);
				ConstraintContainer.ApplyPositionConstraints(RangeIndices, Dt, It, NumIts);
			}
		}

		virtual void ApplyVelocityConstraints(const FReal Dt, const int32 It, const int32 NumIts) override final
		{
			ConstraintContainer.ApplyVelocityConstraints(ConstraintIndices, Dt, It, NumIts);
		}

		virtual void ApplyVelocityConstraints(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex) override final
		{
			check(BeginIndex >= 0);
			check(EndIndex <= GetNumConstraints());

			const int32 RangeSize = EndIndex - BeginIndex;
			if (RangeSize > 0)
			{
				TArrayView<int32> RangeIndices = MakeArrayView(&ConstraintIndices[BeginIndex], RangeSize);
				ConstraintContainer.ApplyVelocityConstraints(RangeIndices, Dt, It, NumIts);
			}
		}

		virtual void ApplyProjectionConstraints(const FReal Dt, const int32 It, const int32 NumIts) override final
		{
			ConstraintContainer.ApplyProjectionConstraints(ConstraintIndices, Dt, It, NumIts);
		}

		virtual void ApplyProjectionConstraints(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex) override final
		{
			check(BeginIndex >= 0);
			check(EndIndex <= GetNumConstraints());

			const int32 RangeSize = EndIndex - BeginIndex;
			if (RangeSize > 0)
			{
				TArrayView<int32> RangeIndices = MakeArrayView(&ConstraintIndices[BeginIndex], RangeSize);
				ConstraintContainer.ApplyProjectionConstraints(RangeIndices, Dt, It, NumIts);
			}
		}

	private:
		virtual void ResetImpl(const int32 MaxConstraints) override final
		{
			ConstraintIndices.Reset(MaxConstraints);
		}

		virtual bool SupportsPartitioning() const override final { return true; }

		virtual bool SortConstraintsByPartitionIndex(const TArray<int32>& PartitionIndices, const TArray<int32>& PrefixSum) override final
		{
			Private::SortArrayByPartitionIndex(ConstraintIndices, PartitionIndices, PrefixSum);
			return true;
		}

		virtual void GetAllConstrainedParticles(TArray<const FGeometryParticleHandle*>& ParticleHandles) const override final
		{
			ParticleHandles.Reset();
			ParticleHandles.Reserve(ConstraintIndices.Num() * 2);

			for (int32 Index : ConstraintIndices)
			{
				const FConstraintHandle* Constraint = ConstraintContainer.GetConstraintHandles()[Index];
				const FParticlePair& Pair = Constraint->GetConstrainedParticles();
				ParticleHandles.Add(Pair[0]);
				ParticleHandles.Add(Pair[1]);
			}
		}

		FConstraintContainerType& ConstraintContainer;
		TArray<int32> ConstraintIndices;
	};


	/**
	* Utility base class for constraint containers usingindexed-based constraint handles (e.g., FJointConstraints)
	*/
	class FPBDIndexedConstraintContainer : public FPBDConstraintContainer
	{
	public:
		FPBDIndexedConstraintContainer(FConstraintHandleTypeID InType)
			: FPBDConstraintContainer(InType)
		{
		}

		virtual void SetConstraintEnabled(int32 ConstraintIndex, bool bEnabled) { }
		virtual bool IsConstraintEnabled(int32 ConstraintIndex) const { return true; }

	protected:
		// @todo(chaos): remove these
		inline int32 GetConstraintIndex(const FIndexedConstraintHandle* ConstraintHandle) const
		{
			return ConstraintHandle->GetConstraintIndex();
		}

		inline void SetConstraintIndex(FIndexedConstraintHandle* ConstraintHandle, int32 ConstraintIndex) const
		{
			ConstraintHandle->ConstraintIndex = ConstraintIndex;
		}
	};

	template<typename ConstaintContainerType>
	class TPBDIndexedConstraintContainer : public FPBDIndexedConstraintContainer
	{
	public:
		using FConstaintContainerType = ConstaintContainerType;

		TPBDIndexedConstraintContainer(FConstraintHandleTypeID InType)
			: FPBDIndexedConstraintContainer(InType)
		{
		}

		// Create RBAN solver: just solves all constraints in the container in the container's order
		virtual TUniquePtr<FConstraintContainerSolver> CreateSceneSolver(const int32 Priority) override final
		{
			return MakeUnique<TSimpleConstraintContainerSolver<FConstaintContainerType>>(static_cast<FConstaintContainerType&>(*this), Priority);
		}

		// Create Main Scene solver: solves constraints in groups determined by the island manager
		virtual TUniquePtr<FConstraintContainerSolver> CreateGroupSolver(const int32 Priority) override final
		{
			return MakeUnique<TIndexedConstraintContainerSolver<FConstaintContainerType>>(static_cast<FConstaintContainerType&>(*this), Priority);
		}
	};

}
