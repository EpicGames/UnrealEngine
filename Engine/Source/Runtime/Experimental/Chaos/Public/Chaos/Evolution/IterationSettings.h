// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"

#include <limits>

namespace Chaos
{
	namespace Private
	{
		/**
		 * Iteration counts for use by bodies and constraints. 
		 * A simulation island will use the maximum number of iterations of any body or constraint in the island.
		 * Iteration counts may be -1 which indicates that the configuration setting should be used.
		*/
		template<typename TIndexType> 
		class TIterationSettings
		{
		private:
			using FIndexType = TIndexType;
			FIndexType NumPositionIterations;
			FIndexType NumVelocityIterations;
			FIndexType NumProjectionIterations;

		public:
			static const int32 InvalidIterations = INDEX_NONE;
			static const int32 MaxIterations = std::numeric_limits<FIndexType>::max();

			static TIterationSettings<FIndexType> MakeEmpty()
			{
				return TIterationSettings<FIndexType>(InvalidIterations, InvalidIterations, InvalidIterations);
			}

			// Merge (take the max) of the two iteration settings objects. Returned object has index type of the first parameter.
			template<typename TOtherIndex>
			static TIterationSettings<FIndexType> Merge(const TIterationSettings<FIndexType>& L, const TIterationSettings<TOtherIndex>& R)
			{
				return TIterationSettings<FIndexType>(
					FMath::Max((int32)L.NumPositionIterations, (int32)R.NumPositionIterations),
					FMath::Max((int32)L.NumVelocityIterations, (int32)R.NumVelocityIterations),
					FMath::Max((int32)L.NumProjectionIterations, (int32)R.NumProjectionIterations)
				);
			}

			TIterationSettings<FIndexType>()
				: NumPositionIterations(InvalidIterations)
				, NumVelocityIterations(InvalidIterations)
				, NumProjectionIterations(InvalidIterations)
			{
			}

			TIterationSettings<FIndexType>(const int32 InNumPositionIterations, const int32 InNumVelocityIterations, const int32 InNumProjectionInterations)
			{
				SetNumPositionIterations(InNumPositionIterations);
				SetNumVelocityIterations(InNumVelocityIterations);
				SetNumProjectionIterations(InNumProjectionInterations);
			}

			int32 GetNumPositionIterations() const { return NumPositionIterations; }
			int32 GetNumVelocityIterations() const { return NumVelocityIterations; }
			int32 GetNumProjectionIterations() const { return NumProjectionIterations; }

			void SetNumPositionIterations(const int32 InNum) { NumPositionIterations = FIndexType(FMath::Clamp(InNum, InvalidIterations, MaxIterations)); }
			void SetNumVelocityIterations(const int32 InNum) { NumVelocityIterations = FIndexType(FMath::Clamp(InNum, InvalidIterations, MaxIterations)); }
			void SetNumProjectionIterations(const int32 InNum) { NumProjectionIterations = FIndexType(FMath::Clamp(InNum, InvalidIterations, MaxIterations)); }

			friend bool operator==(const TIterationSettings& L, const TIterationSettings& R)
			{
				return (L.NumPositionIterations == R.NumPositionIterations) && (L.NumVelocityIterations == R.NumVelocityIterations) && (L.NumProjectionIterations == R.NumProjectionIterations);
			}

			friend bool operator!=(const TIterationSettings& L, const TIterationSettings& R)
			{
				return !(L == R);
			}
		};

		// NOTE: only use signed types (or modify how we handle TIterationSettings::InvalidIterations above)
		using FIterationSettings8 = TIterationSettings<int8>;
		using FIterationSettings16 = TIterationSettings<int16>;

		using FIterationSettings = FIterationSettings8;
	}
}
