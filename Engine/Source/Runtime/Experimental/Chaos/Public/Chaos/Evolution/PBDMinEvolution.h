// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/ArrayCollectionArray.h"
#include "Chaos/Collision/CollisionApplyType.h"
#include "Chaos/Evolution/ConstraintGroupSolver.h"
#include "Chaos/Evolution/SimulationSpace.h"
#include "Chaos/Evolution/SolverBodyContainer.h"
#include "Chaos/Evolution/IterationSettings.h"
#include "Chaos/ParticleHandleFwd.h"


namespace Chaos
{
	class FBasicCollisionDetector;
	class FConstraintContainerSolver;
	class FPBDCollisionConstraints;
	class FSimpleConstraintRule;
	class FPBDRigidsSOAs;

	/**
	 * A minimal optimized evolution with support for
	 *	- PBD Rigids
	 *	- Joints
	 *	- Collisions
	 *
	 * It is single-threaded and does not use a constraint graph or partition the particles into islands.
	 */
	class FPBDMinEvolution
	{
	public:
		using FCollisionDetector = FBasicCollisionDetector;
		using FEvolutionCallback = TFunction<void()>;
		using FRigidParticleSOAs = FPBDRigidsSOAs;

		CHAOS_API FPBDMinEvolution(FRigidParticleSOAs& InParticles, TArrayCollectionArray<FVec3>& InPrevX, TArrayCollectionArray<FRotation3>& InPrevR, FCollisionDetector& InCollisionDetector);
		CHAOS_API ~FPBDMinEvolution();

		CHAOS_API void AddConstraintContainer(FPBDConstraintContainer& InContainer, const int32 Priority = 0);
		CHAOS_API void SetConstraintContainerPriority(const int32 ContainerId, const int32 Priority);

		CHAOS_API void Advance(const FReal StepDt, const int32 NumSteps, const FReal RewindDt);
		CHAOS_API void AdvanceOneTimeStep(const FReal Dt, const FReal StepFraction);

		void SetNumPositionIterations(const int32 NumIts)
		{
			Private::FIterationSettings Iterations = ConstraintSolver.GetIterationSettings();
			Iterations.SetNumPositionIterations(NumIts);
			ConstraintSolver.SetIterationSettings(Iterations);
		}

		void SetNumVelocityIterations(const int32 NumIts)
		{
			Private::FIterationSettings Iterations = ConstraintSolver.GetIterationSettings();
			Iterations.SetNumVelocityIterations(NumIts);
			ConstraintSolver.SetIterationSettings(Iterations);
		}

		void SetNumProjectionIterations(const int32 NumIts)
		{
			Private::FIterationSettings Iterations = ConstraintSolver.GetIterationSettings();
			Iterations.SetNumProjectionIterations(NumIts);
			ConstraintSolver.SetIterationSettings(Iterations);
		}

		void SetGravity(const FVec3& G)
		{
			Gravity = G;
		}

		void SetRewindVelocities(bool InRewindVelocities)
		{
			bRewindVelocities = InRewindVelocities;
		}

		void SetSimulationSpace(const FSimulationSpace& InSimulationSpace)
		{
			SimulationSpace = InSimulationSpace;
		}

		FSimulationSpaceSettings& GetSimulationSpaceSettings()
		{
			return SimulationSpaceSettings;
		}

		const FSimulationSpaceSettings& GetSimulationSpaceSettings() const
		{
			return SimulationSpaceSettings;
		}

		void SetSimulationSpaceSettings(const FSimulationSpaceSettings& InSimulationSpaceSettings)
		{
			SimulationSpaceSettings = InSimulationSpaceSettings;
		}


		UE_DEPRECATED(5.2, "InBoundsExtension parameter has been removed")
		FPBDMinEvolution(FRigidParticleSOAs& InParticles, TArrayCollectionArray<FVec3>& InPrevX, TArrayCollectionArray<FRotation3>& InPrevR, FCollisionDetector& InCollisionDetector, const FReal InBoundsExtension)
			: FPBDMinEvolution(InParticles, InPrevX, InPrevR, InCollisionDetector)
		{
		}

		UE_DEPRECATED(5.2, "BoundsExtension has been removed")
		void SetBoudsExtension(const FReal Unused)
		{
		}

	private:
		CHAOS_API void PrepareTick();
		CHAOS_API void UnprepareTick();
		CHAOS_API void Rewind(FReal Dt, FReal RewindDt);
		CHAOS_API void Integrate(FReal Dt);
		CHAOS_API void ApplyKinematicTargets(FReal Dt, FReal StepFraction);
		CHAOS_API void DetectCollisions(FReal Dt);
		CHAOS_API void GatherInput(FReal Dt);
		CHAOS_API void ScatterOutput(FReal Dt);
		CHAOS_API void ApplyConstraintsPhase1(FReal Dt);
		CHAOS_API void ApplyConstraintsPhase2(FReal Dt);
		CHAOS_API void ApplyConstraintsPhase3(FReal Dt);

		FRigidParticleSOAs& Particles;
		FCollisionDetector& CollisionDetector;

		TArrayCollectionArray<FVec3>& ParticlePrevXs;
		TArrayCollectionArray<FRotation3>& ParticlePrevRs;

		TArray<FPBDConstraintContainer*> ConstraintContainers;
		Private::FPBDSceneConstraintGroupSolver ConstraintSolver;

		FVec3 Gravity;
		FSimulationSpaceSettings SimulationSpaceSettings;
		FSimulationSpace SimulationSpace;
		bool bRewindVelocities;
	};
}
