// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"

namespace Chaos
{
	/**
	 * Information about the space in which a simulation is being run.
	 * This, when combined with a FSimulationSpaceSettings object, can
	 * be used to pass a tunable fraction of the world-space movement
	 * on to the bodies.
	 */
	class FSimulationSpace
	{
	public:
		FSimulationSpace()
			: LinearAcceleration(0)
			, AngularAcceleration(0)
			, LinearVelocity(0)
			, AngularVelocity(0)
		{
		}

		// All properties should be in simulation space
		FVec3 LinearAcceleration;
		FVec3 AngularAcceleration;
		FVec3 LinearVelocity;
		FVec3 AngularVelocity;
	};

	/**
	 * Settings to control how much of world-space movement
	 * should get passed into the simulation.
	 * If all the Alpha values are set to 1, the simulation
	 * should be the same as a world-space simulation.
	 * Usually you would only change Alpha, but it
	 * is also possible to manipulate the elements of the
	 * phantom forces (like centrifugal force) if you know
	 * what you are doing.
	 *
	 * https://en.wikipedia.org/wiki/Rotating_reference_frame
	 */
	class FSimulationSpaceSettings
	{
	public:
		FSimulationSpaceSettings()
			: MasterAlpha(0)
			, Alpha(0)
			, bEnabled(false)
			, LinearAccelerationAlpha(1)
			, CoriolisAlpha(1)
			, CentrifugalAlpha(1)
			, EulerAlpha(1)
			, AngularAccelerationAlpha(1)
			, LinearVelocityAlpha(1)
			, AngularVelocityAlpha(1)
			, ExternalLinearEtherDrag(FVec3(0))
		{
		}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FSimulationSpaceSettings(const FSimulationSpaceSettings&) = default;
		FSimulationSpaceSettings& operator=(const FSimulationSpaceSettings&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		UE_DEPRECATED(5.1, "MasterAlpha is deprecated - please use bEnabled")
		FReal MasterAlpha;

		UE_DEPRECATED(5.6, "Alpha is deprecated - please use bEnabled")
		FReal Alpha;

		// Enable or disable simulation space.
		bool bEnabled;

		// How much of the simulation frame's linear acceleration to pass onto the particles
		FReal LinearAccelerationAlpha;

		// How much of the coriolis force to apply to the particles
		FReal CoriolisAlpha;

		// How much of the centrifugal force to apply to the particles
		FReal CentrifugalAlpha;

		// How much of the euler force to apply to the particles
		FReal EulerAlpha;

		// How much of the simulation frame's angular acceleration to pass onto the particles
		FReal AngularAccelerationAlpha;

		// How much of the simulation frame's linear velocity to pass onto the particles (linear ether drag)
		FReal LinearVelocityAlpha;

		// How much of the simulation frame's angular velocity to pass onto the particles (angular ether drag)
		FReal AngularVelocityAlpha;

		// An additional linear drag on top of the EtherDrag that can be set on the physics asset. Vector in simulation local space.
		FVec3 ExternalLinearEtherDrag;
	};

}
