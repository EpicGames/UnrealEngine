// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Math/Quat.h"


// Set to 1 to enable extra NaN tests in the constraint solver
// NOTE: These will cause slowness!
#ifndef CHAOS_CONSTRAINTSOLVER_NAN_DIAGNOSTIC
#define CHAOS_CONSTRAINTSOLVER_NAN_DIAGNOSTIC ((DO_CHECK && UE_BUILD_DEBUG) || ENABLE_NAN_DIAGNOSTIC)
#endif

// Set to 1 to force single precision in the constraint solver, even if the default numeric type is double
#define CHAOS_CONSTRAINTSOLVER_LOWPRECISION 1

// Add some NaN checking when CHAOS_CONSTRAINTSOLVER_NAN_DIAGNOSTIC is defined
#if CHAOS_CONSTRAINTSOLVER_NAN_DIAGNOSTIC
inline void ChaosSolverCheckNaN(const Chaos::FRealSingle& V) { ensure(!FMath::IsNaN(V)); }
inline void ChaosSolverCheckNaN(const Chaos::TVec3<Chaos::FRealSingle>& V) { ensure(!V.ContainsNaN()); }
inline void ChaosSolverCheckNaN(const Chaos::TRotation3<Chaos::FRealSingle>& V) { ensure(!V.ContainsNaN()); }

inline void ChaosSolverCheckNaN(const Chaos::FRealDouble& V) { ensure(!FMath::IsNaN(V)); }
inline void ChaosSolverCheckNaN(const Chaos::TVec3<Chaos::FRealDouble>& V) { ensure(!V.ContainsNaN()); }
inline void ChaosSolverCheckNaN(const Chaos::TRotation3<Chaos::FRealDouble>& V) { ensure(!V.ContainsNaN()); }
#else
#define ChaosSolverCheckNaN(...)
#endif


namespace Chaos
{
	// Set the math types used by the constraint solvers when single precision is acceptable.
	// NOTE: public APIs should still use FReal, FVec3 etc. whereas FSolverReal and associated
	// types are for use internally where double precision is unnecessary and causes performance
	// issues and/or memory bloat.
#if CHAOS_CONSTRAINTSOLVER_LOWPRECISION
	using FSolverReal = FRealSingle;
	using SolverVectorRegister = VectorRegister4Float;
#else
	using FSolverReal = FReal;
	using SolverVectorRegister = VectorRegister4;
#endif
	using FSolverVec3 = TVec3<FSolverReal>;
	using FSolverRotation3 = TRotation3<FSolverReal>;
	using FSolverMatrix33 = TMatrix33<FSolverReal>;

	class FSolverBody;
	class FSolverBodyContainer;


	// A pair of pointers to solver bodies
	// @note Pointers are only valid for the Constraint Solving phase of the tick
	using FSolverBodyPtrPair = TVector<FSolverBody*, 2>;

	/**
	 * @brief An approximate quaternion normalize for use in the solver
	 * 
	 * @note we need to correctly normalize the final quaternion before pushing it back to the
	 * particle otherwise some tolerance checks elsewhere will fail (Integrate)
	 * 
	 * This avoids the sqrt which is a massively dominating cost especially with doubles
	 * when we do not have a fast reciproical sqrt (AVX2)
	 *
	 * This uses the first order Pade approximation instead of a Taylor expansion
	 * to get more accurate results for small quaternion deltas (i.e., where
	 * Q.SizeSquared() is already near 1). 
	 *
	 * Q.Normalized ~= Q * (2 / (1 + Q.SizeSquared)))
	 * 
	 * In practice we can use this for almost any delta generated in collision detection
	 * but we have an accurate fallback just in case. The fallback adds a branch but this 
	 * does not seem to cost much.
	 * 
	*/
	CHAOS_API void SolverQuaternionNormalizeApprox(FRotation3& InOutQ);

	CHAOS_API FRotation3 SolverQuaternionApplyAngularDeltaApprox(const FRotation3& InQ0, const FVec3& InDR);


	/**
	 * Used by the constraint solver loop to cache all state for a particle and accumulate solver results.
	 * Uses a gather/scatter mechanism to read/write data to the particle SOAs at the beginning/end of the constraint solve.
	 * Constraint solver algorithms, and collision Update functions are implemented to use FSolverBody, and do not 
	 * directly read/write to the particle handles. Constraint Solvers will modify P(), Q(), V() and W() via 
	 * ApplyTransformDelta() and other methods.
	 * 
	 * There is one SolverBody for each particle in an island. Most constraint solvers will actually wrap the
	 * FSolverBody in FConstraintSolverBody, which allows us to apply per-constraint modifiers to the Solver Body.
	 *
	 * @note the X(), P(), R(), Q() accessors on FSolverBody return the Center of Mass positions and rotations, in contrast
	 * to the Particle methods which gives Actor positions and rotations. This is because the Constraint Solvers all calculate
	 * impulses and position corrections relative to the center of mass.
	 * 
	 * @todo(chaos): layout for cache
	 * 
	 */
	class FSolverBody
	{
	public:
		static constexpr FSolverReal ZeroMassThreshold() { return std::numeric_limits<FSolverReal>::min(); }


		/**
		 * A factory method to create a safely initialized Solverbody. 
		 * The defaultconstructor assumes you are going to set all properties manually.
		 */
		static FSolverBody MakeInitialized()
		{
			FSolverBody SolverBody;
			SolverBody.State.Init();
			return SolverBody;
		}

		/**
		 * Create an empty solver body. All properties are uninitialized.
		*/
		static FSolverBody MakeUninitialized()
		{
			return FSolverBody();
		}

		/**
		 * Create an empty solver body. All properties are uninitialized.
		*/
		FSolverBody() {}

		/**
		 * Reset the solver accumulators
		 */
		void Reset()
		{
			State.DP = FSolverVec3(0);
			State.DQ = FSolverVec3(0);
			State.CP = FSolverVec3(0);
			State.CQ = FSolverVec3(0);
		}

		/**
		 * @brief Calculate and set the velocity and angular velocity from the net transform delta
		*/
		inline void SetImplicitVelocity(FReal Dt)
		{
			if (IsDynamic() && (Dt != FReal(0)))
			{
				const FSolverReal InvDt = FSolverReal(1) / FSolverReal(Dt);
				SetV(State.V + FVec3((State.DP - State.CP) * InvDt));
				SetW(State.W + FVec3((State.DQ - State.CQ) * InvDt));
			}
		}

		/**
		 * @brief Get the inverse mass
		*/
		//inline FReal InvM() const { return State.InvM; }
		inline FSolverReal InvM() const { return State.InvM; }

		/**
		 * @brief Set the inverse mass
		*/
		inline void SetInvM(FReal InInvM) { State.InvM = FSolverReal(InInvM); }

		/**
		 * @brief Get the world-space inverse inertia
		*/
		inline const FSolverMatrix33& InvI() const { return State.InvI; }

		/**
		 * @brief Set the world-space inverse inertia
		*/
		inline void SetInvI(const FMatrix33& InInvI) { State.InvI = FSolverMatrix33(InInvI); }

		/**
		 * @brief Get the local-space inverse inertia (diagonal elements)
		*/
		inline const FSolverVec3& InvILocal() const { return State.InvILocal; }

		/**
		 * @brief Set the local-space inverse inertia (diagonal elements)
		*/
		inline void SetInvILocal(const FVec3& InInvILocal)
		{ 
			State.InvILocal = FSolverVec3(InInvILocal); 
			UpdateRotationDependentState();
		}

		/**
		 * @brief The current CoM transform
		*/
		inline FRigidTransform3 CoMTransform() const { return FRigidTransform3(P(), Q()); }

		/**
		 * @brief Pre-integration world-space center of mass position
		*/
		inline const FVec3& X() const { return State.X; }
		inline void SetX(const FVec3& InX) { State.X = InX; }

		/**
		 * @brief Pre-integration world-space center of mass rotation
		*/
		inline const FRotation3& R() const { return State.R; }
		inline void SetR(const FRotation3& InR)
		{
			ChaosSolverCheckNaN(InR);
			State.R = InR;
		}

		/**
		 * @brief Predicted (post-integrate) world-space center of mass position
		 * @note This does not get updated as we iterate
		 * @see DP(), CorrectedP()
		*/
		inline const FVec3& P() const { return State.P; }
		inline void SetP(const FVec3& InP) { State.P = InP; }

		/**
		 * @brief Predicted (post-integrate) world-space center of mass rotation
		 * @note This does not get updated as we iterate
		 * @see DQ(), CorrectedQ()
		*/
		inline const FRotation3& Q() const { return State.Q; }
		inline void SetQ(const FRotation3& InQ)
		{ 
			ChaosSolverCheckNaN(InQ);
			State.Q = InQ;
		}

		/**
		 * @brief World-space center of mass velocity
		*/
		inline const FSolverVec3& V() const { return State.V; }
		inline void SetV(const FVec3& InV)
		{ 
			ChaosSolverCheckNaN(InV);
			State.V = FSolverVec3(InV);
		}

		/**
		 * @brief World-space center of mass angular velocity
		*/
		inline const FSolverVec3& W() const { return State.W; }
		inline void SetW(const FVec3& InW)
		{
			ChaosSolverCheckNaN(InW);
			State.W = FSolverVec3(InW);
		}

		inline const FVec3& CoM() const { return State.CoM; }
		inline void SetCoM(const FVec3& InCoM) { State.CoM = InCoM; }

		inline const FRotation3& RoM() const { return State.RoM; }
		inline void SetRoM(const FRotation3& InRoM) { State.RoM = InRoM; }

		/**
		 * @brief Net world-space position displacement applied by the constraints
		*/
		inline const FSolverVec3& DP() const { return State.DP; }
		inline void SetDP(const FSolverVec3& InDP) { State.DP = InDP; }

		/**
		 * @brief Net world-space rotation displacement applied by the constraints (axis-angle vector equivalent to angular velocity but for position)
		*/
		inline const FSolverVec3& DQ() const { return State.DQ; }
		inline void SetDQ(const FSolverVec3& InDQ) { State.DQ = InDQ; }

		/**
		 * @brief Net world-space position correction applied by the constraints
		 * @note This only includes correction terms that do not introduce linear velocity. The full solver displacement is given by DP
		*/
		inline const FSolverVec3& CP() const { return State.CP; }

		/**
		 * @brief Net world-space rotation correction applied by the constraints (axis-angle vector equivalent to angular velocity but for position)
		 * @note This only includes correction terms that do not introduce angular velocity. The full solver angular displacement is given by DQ
		*/
		inline const FSolverVec3& CQ() const { return State.CQ; }

		/**
		 * @brief World-space position after applying the net correction DP()
		 * @note Calculated on demand from P() and DP() (only requires vector addition)
		*/
		inline FVec3 CorrectedP() const { return State.P + FVec3(State.DP); }

		/**
		 * @brief World-space rotation after applying the net correction DQ()
		 * @note Calculated on demand from Q() and DQ() (requires quaternion multiply and normalization)
		*/
		inline FRotation3 CorrectedQ() const { return (IsDynamic() && !State.DQ.IsZero()) ? FRotation3::IntegrateRotationWithAngularVelocity(State.Q, FVec3(State.DQ), FReal(1)) : State.Q; }

		/**
		 * @brief Apply the accumulated position and rotation corrections to the predicted P and Q
		 * This is only used by unit tests that reuse solver bodies between ticks
		 * @todo(chaos): fix the unit tests (FJointSolverTest::Tick) and remove this
		*/
		void ApplyCorrections()
		{
			State.P = CorrectedP();
			State.Q = CorrectedQ();
			State.DP = FSolverVec3(0);
			State.DQ = FSolverVec3(0);
			State.CP = FSolverVec3(0);
			State.CQ = FSolverVec3(0);
		}

		/**
		 * @brief Get the world-space Actor position 
		*/
		inline FVec3 ActorP() const { return P() - FVec3(ActorQ().RotateVector(CoM())); }

		/**
		 * @brief Get the world-space Actor rotation 
		*/
		inline FRotation3 ActorQ() const { return Q() * RoM().Inverse(); }

		/**
		 * @brief Get the current world-space Actor position 
		 * @note This is recalculated from the current CoM transform including the accumulated position and rotation corrections.
		*/
		inline FVec3 CorrectedActorP() const { return CorrectedP() - CorrectedActorQ().RotateVector(CoM()); }

		/**
		 * @brief Get the current world-space Actor rotation
		 * @note This is recalculated from the current CoM transform including the accumulated position and rotation corrections.
		*/
		inline FRotation3 CorrectedActorQ() const { return CorrectedQ() * RoM().Inverse(); }

		/**
		 * @brief Contact graph level. This is used in shock propagation to determine which of two bodies should have its inverse mass scaled
		*/
		inline int32 Level() const { return State.Level; }
		inline void SetLevel(int32 InLevel) { State.Level = InLevel; }

		/**
		 * @brief Whether the body has a finite mass
		 * @note This is based on the current inverse mass, so a "dynamic" particle with 0 inverse mass will return true here.
		*/
		inline bool IsDynamic() const { return (State.InvM > FSolverBody::ZeroMassThreshold()); }

		/**
		 * @brief Apply a world-space position and rotation delta to the body center of mass, and update inverse mass
		*/
		inline void ApplyTransformDelta(const FSolverVec3& DP, const FSolverVec3& DR)
		{
			ApplyPositionDelta(DP);
			ApplyRotationDelta(DR);
		}

		/**
		 * @brief Apply a world-space position delta to the solver body center of mass
		*/
		inline void ApplyPositionDelta(const FSolverVec3& DP)
		{
			ChaosSolverCheckNaN(DP);
			State.DP += DP;
		}

		/**
		 * @brief Apply a world-space rotation delta to the solver body and update the inverse mass
		*/
		inline void ApplyRotationDelta(const FSolverVec3& DR)
		{
			ChaosSolverCheckNaN(DR);
			State.DQ += DR;
		}

		/**
		 * @brief Apply a world-space position correction delta to the solver body center of mass
		 * This will translate the body without introducing linear velocity
		*/
		inline void ApplyPositionCorrectionDelta(const FSolverVec3& CP)
		{
			ChaosSolverCheckNaN(CP);
			State.CP += CP;
			State.DP += CP;
		}

		/**
		 * @brief Apply a world-space rotation correction delta to the solver body
		 * This will rotate the body without introducing angular velocity
		*/
		inline void ApplyRotationCorrectionDelta(const FSolverVec3& CR)
		{
			ChaosSolverCheckNaN(CR);
			State.CQ += CR;
			State.DQ += CR;
		}

		/**
		 * @brief Apply a world-space velocity delta to the solver body
		*/
		inline void ApplyVelocityDelta(const FSolverVec3& DV, const FSolverVec3& DW)
		{
			ApplyLinearVelocityDelta(DV);
			ApplyAngularVelocityDelta(DW);
		}

		/**
		 * @brief Apply a world-space linear velocity delta to the solver body
		*/
		inline void ApplyLinearVelocityDelta(const FSolverVec3& DV)
		{
			ChaosSolverCheckNaN(DV);
			State.V += DV;
		}

		/**
		 * @brief Apply an world-space angular velocity delta to the solver body
		*/
		inline void ApplyAngularVelocityDelta(const FSolverVec3& DW)
		{
			ChaosSolverCheckNaN(DW);
			SetW(State.W + DW);
		}

		/**
		 * @brief Update the rotation to be in the same hemisphere as the provided quaternion.
		 * This is used by joints with angular constraint/drives
		*/
		inline void EnforceShortestRotationTo(const FRotation3& InQ)
		{
			State.Q.EnforceShortestArcWith(InQ);
		}

		/**
		 * @brief Update cached state that depends on rotation (i.e., world space inertia)
		*/
		void UpdateRotationDependentState();



		void PrefetchPositionSolverData() const
		{
			// The position solver only uses DP,DQ
			FPlatformMisc::PrefetchBlock(&State.DP, 8 * sizeof(float));
		}

		void PrefetchVelocitySolverData() const
		{
			// The velocity solver only uses V,W
			FPlatformMisc::PrefetchBlock(&State.V, 8 * sizeof(float));
		}

	private:

		// The struct exists only so that we can use the variable names
		// as accessor names without violation the variable naming convention
		struct FState
		{
			FState()
			{}

			void Init()
			{
				DP = FSolverVec3(0);
				DQ = FSolverVec3(0);
				V = FSolverVec3(0);
				W = FSolverVec3(0);
				InvI = FSolverMatrix33(0);
				RoM = FRotation3::FromIdentity();
				R = FRotation3::FromIdentity();
				Q = FRotation3::FromIdentity();
				CoM = FVec3(0);
				X = FVec3(0);
				P = FVec3(0);
				InvILocal = FSolverVec3(0);
				InvM = 0;
				CP = FSolverVec3(0);
				CQ = FSolverVec3(0);
				Level = 0;
			}

			// Net position delta applied by all constraints (constantly changing as we iterate over constraints)
			alignas(16) FSolverVec3 DP;

			// Net rotation delta applied by all constraints (constantly changing as we iterate over constraints)
			alignas(16) FSolverVec3 DQ;

			// World-space center of mass velocity
			alignas(16) FSolverVec3 V;

			// World-space center of mass angular velocity
			alignas(16) FSolverVec3 W;

			// World-space inverse inertia
			// NOTE: Matrix and Rotation are alignas(16) so beware of padding
			// @todo(chaos): do we need this, or should we force all systems to use the FConstraintSolverBody decorator?
			FSolverMatrix33 InvI;

			// Actor-space center of mass rotation
			FRotation3 RoM;

			// World-space rotation of mass at start of sub step
			FRotation3 R;

			// Predicted world-space center of mass rotation (post-integration, pre-constraint-solve)
			FRotation3 Q;

			// Actor-space center of mass location
			FVec3 CoM;

			// World-space center of mass state at start of sub step
			FVec3 X;

			// Predicted world-space center of mass position (post-integration, pre-constraint-solve)
			FVec3 P;



			// Local-space inverse inertia (diagonal, so only 3 elements)
			FSolverVec3 InvILocal;

			// Inverse mass
			FSolverReal InvM;

			// Net position correction delta applied by all constraints (constantly changing as we iterate over constraints)
			// Will translate the body without introducing linear velocity
			FSolverVec3 CP;

			// Net rotation correction delta applied by all constraints (constantly changing as we iterate over constraints)
			// Will rotate the body without introducing angular velocity
			FSolverVec3 CQ;

			// Distance to a kinematic body (through the contact graph). Used by collision shock propagation
			int32 Level;
		};

		FState State;
	};


	/**
	 * An FSolverBody decorator for adding mass modifiers to a SolverBody. This will scale the
	 * inverse mass and inverse inertia using the supplied scale. It also updates IsDynamic() to
	 * return false if the scaled inverse mass is zero.
	 * 
	 * See FSolverBody for comments on methods.
	 * 
	 * @note This functionality cannot be in FSolverBody because two constraints referencing
	 * the same body may be applying different mass modifiers (e.g., Joints support "bParentDominates"
	 * which is a per-constraint property, not a per-body property.
	 */
	class FConstraintSolverBody
	{
	public:
		FConstraintSolverBody()
			: Body(nullptr)
		{
		}

		FConstraintSolverBody(FSolverBody& InBody)
			: Body(&InBody)
		{
		}

		/**
		 * @brief True if we have been set up to decorate a SolverBody
		*/
		inline bool IsValid() const { return Body != nullptr; }

		/**
		 * @brief Invalidate the solver body reference
		*/
		inline void Reset() { Body = nullptr; }

		/**
		 * @brief Initialize all properties to safe values
		*/
		void Init() { State.Init(); }

		/**
		 * @brief Set the inner solver body (hold by reference)
		 * @note Does not initialize any other properties
		*/
		void SetSolverBody(FSolverBody& InSolverBody)
		{
			Body = &InSolverBody;
		}

		/**
		 * @brief The decorated SolverBody
		*/
		inline FSolverBody& SolverBody() { check(IsValid()); return *Body; }
		inline const FSolverBody& SolverBody() const { check(IsValid()); return *Body; }

		/**
		 * @brief A scale applied to inverse mass
		*/
		inline FSolverReal InvMScale() const { return State.InvMScale; }
		inline void SetInvMScale(FReal InValue) { State.InvMScale = FSolverReal(InValue); }

		/**
		 * @brief A scale applied to inverse inertia
		*/
		inline FSolverReal InvIScale() const { return State.InvIScale; }
		inline void SetInvIScale(FReal InValue) { State.InvIScale = FSolverReal(InValue); }


		/**
		 * @brief Shock propagation mass and inertia scaling (0 for infinite mass, 1 for no effect)
		*/
		inline FSolverReal ShockPropagationScale() const { return State.ShockPropagationScale; }
		inline void SetShockPropagationScale(FReal InValue) { State.ShockPropagationScale = FSolverReal(InValue); }

		/**
		 * @brief The net scaled inverse mass
		*/
		FSolverReal InvM() const { return (State.ShockPropagationScale * State.InvMScale) * Body->InvM(); }

		/**
		 * @brief The net scaled inverse inertia
		*/
		FSolverMatrix33 InvI() const { return (State.ShockPropagationScale * State.InvIScale) * Body->InvI(); }

		/**
		 * @brief The net scaled local space inverse inertia
		*/
		FSolverVec3 InvILocal() const { return (State.ShockPropagationScale * State.InvIScale) * Body->InvILocal(); }

		/**
		 * @brief Whether the body is dynamic (i.e., has a finite mass) after scaling is applied
		*/
		inline bool IsDynamic() const { return (InvM() > FSolverBody::ZeroMassThreshold()); }

		//
		// From here all methods just forward to the FSolverBody
		//

		inline void SetImplicitVelocity(FReal Dt) { Body->SetImplicitVelocity(Dt); }
		inline FRigidTransform3 CoMTransform() const { return Body->CoMTransform(); }
		inline const FVec3& X() const { return Body->X(); }
		inline const FRotation3& R() const { return Body->R(); }
		inline const FVec3& P() const { return Body->P(); }
		inline const FRotation3& Q() const { return Body->Q(); }
		inline const FVec3 ActorP() const { return Body->ActorP(); }
		inline const FRotation3 ActorQ() const { return Body->ActorQ(); }
		inline const FVec3 CorrectedActorP() const { return Body->CorrectedActorP(); }
		inline const FRotation3 CorrectedActorQ() const { return Body->CorrectedActorQ(); }
		inline const FSolverVec3& V() const { return Body->V(); }
		inline const FSolverVec3& W() const { return Body->W(); }
		inline int32 Level() const { return Body->Level(); }
		inline const FSolverVec3& DP() const { return Body->DP(); }
		inline const FSolverVec3& DQ() const { return Body->DQ(); }
		inline const FSolverVec3& CP() const { return Body->CP(); }
		inline const FSolverVec3& CQ() const { return Body->CQ(); }
		inline FVec3 CorrectedP() const { return Body->CorrectedP(); }
		inline FRotation3 CorrectedQ() const { return Body->CorrectedQ(); }

		inline void ApplyTransformDelta(const FSolverVec3& DP, const FSolverVec3& DR) { Body->ApplyTransformDelta(DP, DR); }
		inline void ApplyPositionDelta(const FSolverVec3& DP) { Body->ApplyPositionDelta(DP); }
		inline void ApplyRotationDelta(const FSolverVec3& DR) { Body->ApplyRotationDelta(DR); }
		inline void ApplyPositionCorrectionDelta(const FSolverVec3& DP) { Body->ApplyPositionCorrectionDelta(DP); }
		inline void ApplyRotationCorrectionDelta(const FSolverVec3& DR) { Body->ApplyRotationCorrectionDelta(DR); }
		inline void ApplyVelocityDelta(const FSolverVec3& DV, const FSolverVec3& DW) { Body->ApplyVelocityDelta(DV, DW); }
		inline void ApplyLinearVelocityDelta(const FSolverVec3& DV) { Body->ApplyLinearVelocityDelta(DV); }
		inline void ApplyAngularVelocityDelta(const FSolverVec3& DW) { Body->ApplyAngularVelocityDelta(DW); }
		inline void EnforceShortestRotationTo(const FRotation3& InQ) { Body->EnforceShortestRotationTo(InQ); }
		inline void UpdateRotationDependentState() { Body->UpdateRotationDependentState(); }

	private:
		// Struct is only so that we can use the same var names as function names
		struct FState
		{
			FState() 
			{
			}

			void Init()
			{
				InvMScale = FSolverReal(1);
				InvIScale = FSolverReal(1);
				ShockPropagationScale = FSolverReal(1);
			}

			FSolverReal InvMScale;
			FSolverReal InvIScale;
			FSolverReal ShockPropagationScale;
		};

		// The body we decorate
		FSolverBody* Body;

		// The body modifiers
		FState State;
	};
}
