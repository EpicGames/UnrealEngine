// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Evolution/SolverBody.h"
#include "Chaos/Framework/ContainerItemHandle.h"
#include "Chaos/ObjectPool.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/Particle/ParticleUtilities.h"
#include "Chaos/Framework/ChunkedArray.h"

#include "Containers/Array.h"
#include "Containers/Map.h"

namespace Chaos
{
	class FSolverBody;
	class FSolverBodyContainer;

	/**
	 * The SolverBodies for a set of particles.
	 * 
	 * Each IslandGroup owns a SolverBodyContainer containing the data required for solving the constraints in the 
	 * group's islands. Constraints will hold pointers to SolverBodies in the Island's SolverBodyContainer.
	 * ConstraintSolvers read and write to their SolverBodies and do not access particles directly.
	 * 
	 * SolverBodies are created at the start of the constraint solving phase, and destroyed at the end. They are
	 * stored in a semi-contiguous (chunked) array in roughly the order that they are accessed for cache efficiency. 
	 * Pointers to SolverBodies are persistent for the duration of the tick, but no longer.
	 * 
	 * @note This container holds all the state of bodies in an island. Dynamics will only appear in one island, 
	 * but kinematics are in multiple and so their state will be duplicated in each island. This is ok - we do 
	 * not write to the state of any kinematic bodies in the constraint solvers. We also assume that the number
	 * of kinematics is small compared to the number of dynamics. If this is commonly untrue we may want to 
	 * consider having a separate (global) container of kinematic solver bodies.
	 */
	class FSolverBodyContainer
	{
	private:
		// Settings for the Solver Body array.
		static const int32 NumBodiesPerChunk = 256;
		static const int32 BodyArrayChunkSize = NumBodiesPerChunk * sizeof(FSolverBody);

	public:
		using FSolverBodyArray = Private::TChaosChunkedArray<FSolverBody, BodyArrayChunkSize>;

		FSolverBodyContainer()
			: bLocked(true)
		{
		}

		// Clear the bodies array and allocate enough space for at least MaxBodies bodies.
		// @param MaxBodies The number of bodies that will get added to the container. The container asserts if we attempt to add more than this.
		inline void Reset(int MaxBodies)
		{
			SolverBodies.Reset();
			Particles.Reset(MaxBodies);
			ParticleToIndexMap.Reset();
			bLocked = false;
		}

		// The number of bodies in the container
		inline int Num() const
		{
			return SolverBodies.Num();
		}

		// The maximum number of bodies the container can hold (until Reset() is called again)
		inline int Max() const
		{
			return TNumericLimits<int32>::Max();
		}

		// Get the particle associated with the solver body at Index
		inline FGeometryParticleHandle* GetParticle(const int32 Index) const
		{
			return Particles[Index];
		}

		// Get the solver body at Index
		inline const FSolverBody& GetSolverBody(const int32 Index) const
		{
			return SolverBodies[Index];
		}

		// Get the solver body at Index
		inline FSolverBody& GetSolverBody(const int32 Index)
		{
			return SolverBodies[Index];
		}

		// Add a solver body to represent the solver state of the particle
		// This should ideally be called in the order in which the bodies will be accessed
		// (or as close as we can get, given most constraints access 2 bodies so there is no perfect order)
		// NOTE: This does not populate the solver body with any data from the particle - we must also call
		// GatherInput for that to happen (GatherInput is highly parallizable but FindOrAdd is not)
		FSolverBody* FindOrAdd(FGenericParticleHandle InParticle);

		// Collect all the data we need from the particles represented by our SolverBodies
		void GatherInput(const FReal Dt, const int32 BeginIndex, const int32 EndIndex);

		// Scatter the solver results back to the particles represented by our SolverBodies. Range version used for parallization.
		void ScatterOutput(const int32 BeginIndex, const int32 EndIndex);

		// Recalculate the velocities of all bodies based on their transform deltas
		void SetImplicitVelocities(FReal Dt);

		// Apply accumulated transform deltas to the body transforms
		void ApplyCorrections();

		// Can be called after ApplyCorrections to update inertia to match the current transform
		void UpdateRotationDependentState();

		// Indicate that no more bodies should be added until the next tick (when Reset is called).
		// Used to ensure that no bodies are added in parallel calls to GatherInput.
		void Lock() 
		{ 
			bLocked = true;
		}

		// Used to AB test solvers - not intended for general use (very expensive!)
		void CopyTo(FSolverBodyContainer& Dest)
		{
			Dest.SolverBodies = SolverBodies;
			Dest.Particles = Particles;
			Dest.ParticleToIndexMap = ParticleToIndexMap;
			Dest.bLocked = bLocked;
		}

	private:
		int32 AddParticle(FGenericParticleHandle InParticle);

		// All the solver bodies
		FSolverBodyArray SolverBodies;

		// The particles that map to each solver body
		TArray<FGeometryParticleHandle*> Particles;

		// Used to determine if a (non-dynamic) body is already present in the container. 
		// Dynamic bodies are not in this map - the particle stores its index in this case because
		// dynamic bodies can only be in one island whereas non-dynamics may be const-referenced in many.
		TMap<FGenericParticleHandle, int32> ParticleToIndexMap;

		// To trap incorrect use of the solve bodies. We do not support adding bodies once we have started
		// to call GatherInput because GatherInput is parallelized but FindOrAdd is not
		bool bLocked;
	};

}
