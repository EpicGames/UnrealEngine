// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/ConstraintHandle.h"
#include "Chaos/Island/IslandManagerFwd.h"
#include "Chaos/Evolution/SolverPartitionManager.h"


namespace Chaos
{
	class FConstraintHandleHolder;
	class FSolverBodyContainer;

	namespace CVars
	{
		extern int32 Chaos_SolverPartitionManager_Partitioning;
	}

	/** 
	 * Base class for all the solver for a set of constraints of a specific type.
	 * 
	 * A SolverContainer is used to solve a set constraints in sequential order. There will
	 * be one solver container for each thread on which we solve constraints (see FPBDIslandConstraintGroupSolver). 
	 * How constraints are assigned to groups depends on the constraint type and settings, but
	 * usually a group contains all constraints from one or more islands (unless we are coloring).
	 * 
	 * NOTE: there are two main use-case for FConstraintContainerSolver objects: the main physics scene and RBAN. 
	 * The main scene uses the IslandManager to break the scene up into groups of constraints
	 * that can be solved in parallel (Islands). Those islands are then put into IslandGroups, and each IslandGroup
	 * is solved in a task, therefore we will have one FConstraintContainerSolver per type of constraint per IslandGroup.
	 * RBAN does not attempt to partition its scene into islands and just solves all constraints on its main
	 * thread, so it only has one FConstraintContainerSolver (per constraint type).
	*/
	class FConstraintContainerSolver
	{
	public:

		FConstraintContainerSolver(const int32 InPriority)
			: Priority(InPriority)
			, SolverPartitionManager(nullptr)
		{
		}

		virtual ~FConstraintContainerSolver() = default;

		/**
		 * Set the solver priority.
		 * Solvers are sorted by priority. Lower values are solved first so solvers with higher
		 * priorty values with "win" over lower ones.
		 * @see FPBDConstraintGroupSolver
		*/
		void SetPriority(const int32 InPriority)
		{
			Priority = InPriority;
		}

		/**
		 * Get the solver priority
		*/
		int32 GetPriority() const
		{
			return Priority;
		}

		/**
		 * Set the maximum number of constraints the solver will have to handle.
		 * This will be called only once per tick, so containers resized here will not have to resize again this tick
		 * so that pointers to elements in the container will remain valid for the tick (but not beyond).
		*/
		void Reset(const int32 MaxConstraints)
		{
			ResetImpl(MaxConstraints);
			UpdateSolverPartitionManager();
		}

		virtual int32 GetNumConstraints() const = 0;

		/**
		 * RBAN API.
		 * Add all (active) constraints to the solver.
		 */
		virtual void AddConstraints() = 0;

		/**
		 * Island API.
		 * Add a set of constraints to the solver. This can be called multiple times: once for each island in an IslandGroup, but
		 * there will never be more constraints added than specified in Reset().
		 * NOTE: this should not do any actual data gathering - it should just add to the list of constraints in this group. All data
		 * gathering is handled in GatherInput.
		*/
		virtual void AddConstraints(const TArrayView<Private::FPBDIslandConstraint*>& Constraints) = 0;

		/**
		 * Add all the required bodies to the body container (required for the constraints added with AddConstraints)
		*/
		virtual void AddBodies(FSolverBodyContainer& SolverBodyContainer) = 0;

		virtual void GatherInput(const FReal Dt) = 0;
		virtual void GatherInput(const FReal Dt, const int32 BeginIndex, const int32 EndIndex) = 0;
		virtual void ScatterOutput(const FReal Dt) = 0;
		virtual void ScatterOutput(const FReal Dt, const int32 BeginIndex, const int32 EndIndex) = 0;

		virtual void PreApplyPositionConstraints(const FReal Dt) { PreApplyPositionConstraints(Dt, 0, GetNumConstraints()); }
		virtual void PreApplyPositionConstraints(const FReal Dt, const int32 BeginIndex, const int32 EndIndex) {}
		virtual void PreApplyVelocityConstraints(const FReal Dt) { PreApplyVelocityConstraints(Dt, 0, GetNumConstraints()); }
		virtual void PreApplyVelocityConstraints(const FReal Dt, const int32 BeginIndex, const int32 EndIndex) {}
		virtual void PreApplyProjectionConstraints(const FReal Dt) { PreApplyProjectionConstraints(Dt, 0, GetNumConstraints()); }
		virtual void PreApplyProjectionConstraints(const FReal Dt, const int32 BeginIndex, const int32 EndIndex) {}

		/**
		 * Apply the position solve to all constraints in the container
		*/
		virtual void ApplyPositionConstraints(const FReal Dt, const int32 It, const int32 NumIts) = 0;
		virtual void ApplyPositionConstraints(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex) = 0;

		/**
		 * Apply the velocity solve to all constraints in the container
		*/
		virtual void ApplyVelocityConstraints(const FReal Dt, const int32 It, const int32 NumIts) = 0;
		virtual void ApplyVelocityConstraints(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex) = 0;

		/**
		 * Apply the projection solve to all constraints in the container
		*/
		virtual void ApplyProjectionConstraints(const FReal Dt, const int32 It, const int32 NumIts) = 0;
		virtual void ApplyProjectionConstraints(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex) = 0;


		/**
		 * Set up the partitions, sort constraints accordingly, and assign constraint indices into the constraint container for serial and parallel partition processing.
		*/
		UE_INTERNAL int32 PrepareSolverPartitions()
		{
			if (SolverPartitionManager != nullptr)
			{
				PRAGMA_DISABLE_INTERNAL_WARNINGS
					return SolverPartitionManager->PreparePartitions();
				PRAGMA_ENABLE_INTERNAL_WARNINGS
			}
			return 0;
		}

		/**
		 * Global start indices into solver's constraint container.
		 * Constraint indices in two separate inner arrays must be processed in series (e.g. ConstraintBatchIndices[i][0] and ConstraintBatchIndices[i+1][0]).
		 * Constraint indices in the same inner array can be processed in parallel (e.g. ConstraintBatchIndices[i][j] and ConstraintBatchIndices[i][j+1]).
		*/
		UE_INTERNAL const TArray<TArray<int32>>& GetConstraintsBatchIndices() const
		{
			if (SolverPartitionManager != nullptr)
			{
				PRAGMA_DISABLE_INTERNAL_WARNINGS
					return SolverPartitionManager->GetConstraintsBatchIndices();
				PRAGMA_ENABLE_INTERNAL_WARNINGS
			}
			static const TArray<TArray<int32>> EmptyArray;
			return EmptyArray;
		}

	private:
		virtual void ResetImpl(const int32 MaxConstraints) = 0;

		void UpdateSolverPartitionManager()
		{
			if (CVars::Chaos_SolverPartitionManager_Partitioning == 1 && SupportsPartitioning())
			{
				// Create a solver partition manager to apply greedy graph coloring
				if (!SolverPartitionManager.IsValid() || SolverPartitionManager->GetType(Private::FSolverPartitionManager::FPasskey()) != Private::ESolverPartitionManagerType::GreedyConstraintColoring)
				{
					SolverPartitionManager = TUniquePtr<Private::FGreedyConstraintColoring>(new	Private::FGreedyConstraintColoring(*this, Private::FSolverPartitionManager::FPasskey()));
				}
			}
			else
			{
				// Clear the solver partition manager (no partitioning)
				SolverPartitionManager.Reset();
			}
		}

		// Does the implemented solver support partitioning (i.e. constraint sorting, task-parallel solving)?
		virtual bool SupportsPartitioning() const = 0;

		// Returns true if the sort constraints function is implemented for the respective container type.
		// This function should only be called before initializing the constraint solvers (in AddBodies) since it will only sort the constraints but not the constraint solvers.
		virtual bool SortConstraintsByPartitionIndex(const TArray<int32>& PartitionIndices, const TArray<int32>& PrefixSum) = 0;

		// Returns an array of particles constrained by all constraints in this container.
		// 2 particles per constraint are returned in the order of the constraints.
		// NOTE: Only implemented by container types which support partitioning.
		virtual void GetAllConstrainedParticles(TArray<const FGeometryParticleHandle*>& ParticleHandles) const = 0;

		int32 Priority;

		// The solver partition manager subdivides the constraints into batches and introduces rules on which to process in series and parallel.
		TUniquePtr<Private::FSolverPartitionManager> SolverPartitionManager;

		friend class Private::FGreedyConstraintColoring;
	};
	
	/**
	 * A constraint solver for use with a simple (non-graph-based) evolution (RBAN) and constraint containers
	 * with builtin-in solvers (Joints, Suspension, but not Collisions).
	 * @see TIndexedConstraintContainerSolver, FPBDCollisionContainerSolver
	 * @todo(chaos): really we should split the base class into group and scene versions. See CreateSceneSolver CreateGroupSolver
	*/
	template<typename ConstraintContainerType>
	class TSimpleConstraintContainerSolver : public FConstraintContainerSolver
	{
	public:
		using FConstraintContainerType = ConstraintContainerType;
		using FConstraintHandleType = typename FConstraintContainerType::FConstraintContainerHandle;

		TSimpleConstraintContainerSolver(FConstraintContainerType& InConstraintContainer, const int32 InPriority)
			: FConstraintContainerSolver(InPriority)
			, ConstraintContainer(InConstraintContainer)
		{
		}

		virtual int32 GetNumConstraints() const override final
		{
			return ConstraintContainer.GetNumConstraints();
		}

		virtual void AddConstraints() override final
		{
			// We solve all constraints in the container in the order it prefers so nothing to do here
		}

		virtual void AddConstraints(const TArrayView<Private::FPBDIslandConstraint*>& Constraints) override final
		{
			// This solver container is for use with the a non-graph evolution. It will not call this function.
			ensure(false);
		}

		virtual void AddBodies(FSolverBodyContainer& SolverBodyContainer) override final
		{
			ConstraintContainer.AddBodies(SolverBodyContainer);
		}

		virtual void GatherInput(const FReal Dt) override final
		{
			ConstraintContainer.GatherInput(Dt);
		}

		virtual void GatherInput(const FReal Dt, const int32 BeginIndex, const int32 EndIndex) override final
		{
			// This solver container is for use with the a non-graph evolution. It will not call this function.
			ensure(false);
		}

		virtual void ScatterOutput(const FReal Dt) override final
		{
			ConstraintContainer.ScatterOutput(Dt);
		}

		virtual void ScatterOutput(const FReal Dt, const int32 BeginIndex, const int32 EndIndex) override final
		{
			// This solver container is for use with the a non-graph evolution. It will not call this function.
			ensure(false);
		}

		virtual void ApplyPositionConstraints(const FReal Dt, const int32 It, const int32 NumIts) override final
		{
			ConstraintContainer.ApplyPositionConstraints(Dt, It, NumIts);
		}

		virtual void ApplyPositionConstraints(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex) override final
		{
			// @todo(Chaos): Do we want to implement a parallelized solver option here?
			ensure(false);
		}

		virtual void ApplyVelocityConstraints(const FReal Dt, const int32 It, const int32 NumIts) override final
		{
			ConstraintContainer.ApplyVelocityConstraints(Dt, It, NumIts);
		}

		virtual void ApplyVelocityConstraints(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex) override final
		{
			// @todo(Chaos): Do we want to implement a parallelized solver option here?
			ensure(false);
		}

		virtual void ApplyProjectionConstraints(const FReal Dt, const int32 It, const int32 NumIts) override final
		{
			ConstraintContainer.ApplyProjectionConstraints(Dt, It, NumIts);
		}

		virtual void ApplyProjectionConstraints(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex) override final
		{
			// @todo(Chaos): Do we want to implement a parallelized solver option here?
			ensure(false);
		}

	protected:
		FConstraintContainerType& ConstraintContainer;

	private:
		virtual void ResetImpl(const int32 MaxConstraints) override final
		{
		}

		virtual bool SupportsPartitioning() const override final { return false; }

		virtual bool SortConstraintsByPartitionIndex(const TArray<int32>& PartitionIndices, const TArray<int32>& PrefixSum) override final
		{
			// @todo(Chaos): Do we want to implement a parallelized solver option here? This would likely mean sorting constraints and handles by color.
			return false;
		}

		virtual void GetAllConstrainedParticles(TArray<const FGeometryParticleHandle*>& ParticleHandles) const override final
		{
			// @todo(Chaos): Do we want to implement a parallelized solver option here?
			ParticleHandles.Reset();
		}
	};
}
