// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Containers/Array.h"
#include "Chaos/ParticleHandle.h"

namespace Chaos
{
	class FConstraintContainerSolver;

	namespace Private
	{
		enum class ESolverPartitionManagerType : uint8
		{
			GreedyConstraintColoring,
		};

		// Stable counting sort: groups Elements into contiguous blocks by partition index, preserving the original relative order of elements within each block.
		// PrefixSum supplies the start index of each partition index block after sorting and used as the starting write position for the sort.
		// T may be any copyable type (pointer, handle, value).
		template <typename T>
		void SortArrayByPartitionIndex(
			TArray<T>& Elements,
			const TArray<int32>& PartitionIndices,
			const TArray<int32>& PrefixSum)
		{
			const int32 NumOfElements = Elements.Num();
			check(PartitionIndices.Num() == NumOfElements);

			// Copy start positions to use as per-bucket write cursors.
			TArray<int32> WriteIdx = PrefixSum;

			TArray<T> Sorted;
			Sorted.SetNumUninitialized(NumOfElements);
			for (int32 Index = 0; Index < NumOfElements; ++Index)
			{
				Sorted[WriteIdx[PartitionIndices[Index]]++] = Elements[Index];
			}

			Elements = MoveTemp(Sorted);
		}

		class FSolverPartitionManager
		{
		public:
			// Passkey idiom: public class with private constructor.
			// Anyone can name this type, but only friend classes can construct it.
			class FPasskey 
			{
				friend class Chaos::FConstraintContainerSolver;
				explicit FPasskey() {}
			};

			/**
			 * Not publicly callable because of passkey with private constructor
			*/
			FSolverPartitionManager(Chaos::FConstraintContainerSolver& ContainerIn, FPasskey Passkey);
			virtual ~FSolverPartitionManager();

			/**
			 * Not publicly callable because of passkey with private constructor
			*/
			virtual ESolverPartitionManagerType GetType(FPasskey Passkey) const = 0;

			/**
			 * Global start indices into solver's constraint container.
			 * Constraint indices in two separate inner arrays must be processed in series (e.g. ConstraintBatchIndices[i][0] and ConstraintBatchIndices[i+1][0]).
			 * Constraint indices in the same inner array can be processed in parallel (e.g. ConstraintBatchIndices[i][j] and ConstraintBatchIndices[i][j+1]).
			 * The constraint solver will launch batches from ConstraintBatchIndices[i][j] to ConstraintBatchIndices[i][j+1].
			*/
			UE_INTERNAL const TArray<TArray<int32>>& GetConstraintsBatchIndices() const;

			/**
			 * Assigns a partition ID for all constraints in Container.
			 * Reorders the constraints in the order of their partition ID.
			 * Fills ConstraintsBatchIndices with the start indices of partitions which will be processed in serial and parallel. 
			*/
			UE_INTERNAL virtual int32 PreparePartitions() = 0;

		protected:
			TArray<TArray<int32>> ConstraintsBatchIndices;
			FConstraintContainerSolver& Container;
		};

		class FGreedyConstraintColoring : public FSolverPartitionManager
		{
		public:
			FGreedyConstraintColoring(Chaos::FConstraintContainerSolver& ContainerIn, FPasskey Passkey);
			virtual ESolverPartitionManagerType GetType(FPasskey Passkey) const override final { return ESolverPartitionManagerType::GreedyConstraintColoring; }

			UE_INTERNAL virtual int32 PreparePartitions() override final;

		private:
			/**
			 * Apply greedy graph coloring to all constraints.
			 * Stable sorts constraints according their color (if implemented for this solver).
			 * The returned array indicates the indices where the color changes.
			*/
			void ColorConstraints();

			/**
			 * Assigns ConstraintsBatchIndices to indicate which batches have to run in series (outer array) and which can run in parallel (inner array)
			 * based on the number of worker threads available and minimum constraints per batch.
			*/
			void AssignParallelBatchIndices();

			/**
			 * Reset all class member variables
			*/
			void Reset();

			/**
			 * Maximum number of colors supported by the algorithm.
			 * If there are no more free colors, the constraint color index will be set to 64 and processed in a serial-only batch.
			 */
			static constexpr int32 MaxColors = 64;

			/**
			 * Particles constrained by all constraints in the Container (inherited).
			 * 2 particles per constraint, i.e. ConstrainedParticles.Num() == 2 * NumConstraints
			 */
			TArray<const FGeometryParticleHandle*> ConstrainedParticles;

			/**
			 * Indices of the particles into SOA 
			 * Will be used in index to AllBodyColors
			 * Kinematic, static, or sleeping particles are INDEX_NONE
			 */
			TArray<int32> BodyIndices;

			/**
			 * uint64 bitmask holds the color indices (as bits) of all constraints connected to a solver body.
			 * Bit N is set if a constraint with color index N is already connected to the body. Supports up to 64 colors.
			 * Color I by the I+1-th bit from right (4-bit example: Color 0 -> 0001, Color 2 -> 0100).
			 * Every particle referenced by a constraint has one unique bitmask.
			*/
			TArray<uint64> BodyColors;

			/**
			 * Color indices of all constraints before sorting.
			*/
			TArray<int32> ConstraintColorIndices;

			/**
			 * Counts the number of occurrences of each color index
			*/
			TArray<int32> CountColors;

			/**
			 * Start index of each color in the fully sorted constraint container. 
			 * This is equivalent to the prefix sum of all colors.
			 */
			TArray<int32> ColorStartIndices;
		};

	}
}