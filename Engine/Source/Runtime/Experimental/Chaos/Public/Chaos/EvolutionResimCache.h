// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ResimCacheBase.h"
#include "Templates/UniquePtr.h"
#include "Chaos/Collision/PBDCollisionConstraint.h"
#include "Chaos/ParticleHandle.h"

namespace Chaos
{
	class FCollisionResimCache;

	class FEvolutionResimCache : public IResimCacheBase
	{
	public:
		FEvolutionResimCache(){}
		virtual ~FEvolutionResimCache() = default;
		void ResetCache()
		{
			// If the number of cached particles is shrinking, reduce the memory allocation but leave room for 10% growth without needing to reallocate
			// Memory will be reallocated if the cached particles grow or shrink by 10%, growing reallocation is done automatically, this logic handles shrink reallocation and leaving room for 10% growth.
			// Example: At 100 particles it's allowed to populate the TMap between 90-110 particles without a memory reallocation.
			const int32 CurrentSize = ParticleToCachedSolve.Num();
			const int32 SizeLeniency = FMath::CeilToInt(0.1f * static_cast<float>(CurrentSize)); // 10% leniency
			ParticleCacheAllocationSize = FMath::Max(CurrentSize, ParticleCacheAllocationSize);
			const int32 ReallocationLimit = FMath::Max(ParticleCacheAllocationSize - (SizeLeniency * 2), 0);

			if (CurrentSize < ReallocationLimit)
			{
				const int32 PreferredSize = CurrentSize + SizeLeniency;

				ParticleToCachedSolve.Empty(PreferredSize);
				ParticleCacheAllocationSize = PreferredSize;
			}
			else
			{
				ParticleToCachedSolve.Reset();
			}

			SavedConstraints.Reset();
			WeakSinglePointConstraints.Reset();
		}

		void SaveParticlePostSolve(const FPBDRigidParticleHandle& Particle)
		{
			FPBDSolveCache& Cache = ParticleToCachedSolve.FindOrAdd(Particle.UniqueIdx());
			Cache.P = Particle.GetP();
			Cache.Q = Particle.GetQ();
			Cache.V = Particle.GetV();
			Cache.W = Particle.GetW();
		}

		bool ReloadParticlePostSolve(FPBDRigidParticleHandle& Particle) const
		{
			//if this function is called it means the particle is in sync, which means we should have a cached value
			const FPBDSolveCache* Cache = ParticleToCachedSolve.Find(Particle.UniqueIdx());
			if (Cache && (Particle.GetP() != Cache->P || Particle.GetQ() != Cache->Q || Particle.GetV() != Cache->V || Particle.GetW() != Cache->W))
			{
				Particle.SetP(Cache->P);
				Particle.SetQ(Cache->Q);
				Particle.SetV(Cache->V);
				Particle.SetW(Cache->W);
				return true;
			}
			return false;
		}

		void SaveConstraints(TArrayView<const FPBDCollisionConstraint* const> CollisionsArray)
		{
			// Copy the constraints into an array
			SavedConstraints.Reset(CollisionsArray.Num());
			for (const FPBDCollisionConstraint* Collision : CollisionsArray)
			{
				SavedConstraints.Emplace(FPBDCollisionConstraint::MakeCopy(*Collision));
			}

			//Create weak handles so we can make sure everything is alive later
			auto SaveArrayHelper = [](auto& Constraints,auto& WeakPairs)
			{
				WeakPairs.Empty(Constraints.Num());
				for(FPBDCollisionConstraint& Constraint : Constraints)
				{
					WeakPairs.Add(FWeakConstraintPair{Constraint.GetParticle0()->WeakParticleHandle(),Constraint.GetParticle1()->WeakParticleHandle()});

					auto* A = Constraint.GetParticle0();
					auto* B = Constraint.GetParticle1();

					//Need to do this on save for any new constraints
					MarkSoftIfDesync(*A,*B);
				}
				check(Constraints.Num() == WeakPairs.Num());
			};

			SaveArrayHelper(SavedConstraints,WeakSinglePointConstraints);
		}

		//Returns all constraints that are still valid (resim can invalidate constraints by either deleting particles, moving particles, etc...)
		const TArray<FPBDCollisionConstraint>& GetAndSanitizeConstraints()
		{
			auto CleanupArrayHelper = [](auto& Constraints,auto& WeakPairs)
			{
				check(Constraints.Num() == WeakPairs.Num());
				for(int32 Idx = Constraints.Num() - 1; Idx >= 0; --Idx)
				{
					FPBDCollisionConstraint& Constraint = Constraints[Idx];
					FWeakConstraintPair& WeakPair = WeakPairs[Idx];

					TGeometryParticleHandle<FReal,3>* A = WeakPair.A.GetHandleUnsafe();
					TGeometryParticleHandle<FReal,3>* B = WeakPair.B.GetHandleUnsafe();

					bool bValidConstraint = A != nullptr && B != nullptr;
					if(bValidConstraint)
					{
						//must do this on get in case particles are no longer constrained (means we won't see them during save above)
						bValidConstraint = !MarkSoftIfDesync(*A,*B);
					}

					if(!bValidConstraint)
					{
						Constraints.RemoveAtSwap(Idx);
						WeakPairs.RemoveAtSwap(Idx);
					}
				}
			};

			CleanupArrayHelper(SavedConstraints,WeakSinglePointConstraints);

			return SavedConstraints;
		}

	private:

		static bool MarkSoftIfDesync(TGeometryParticleHandle<FReal,3>& A,TGeometryParticleHandle<FReal,3>& B)
		{
			if(A.SyncState() == ESyncState::HardDesync || B.SyncState() == ESyncState::HardDesync)
			{
				if(A.SyncState() != ESyncState::HardDesync)
				{
					//Need to resim, but may end up still in sync
					//A.SetSyncState(ESyncState::SoftDesync);
				}

				if(B.SyncState() != ESyncState::HardDesync)
				{
					//Need to resim, but may end up still in sync
					//B.SetSyncState(ESyncState::SoftDesync);
				}

				return true;
			}

			return false;
		}

		//NOTE: You must sanitize this before using. This can contain dangling pointers or invalid constraints
		TArray<FPBDCollisionConstraint> SavedConstraints;

		struct FWeakConstraintPair
		{
			FWeakParticleHandle A;
			FWeakParticleHandle B;
		};

		struct FPBDSolveCache
		{
			FVec3 P;
			FQuat Q;
			FVec3 V;
			FVec3 W;
		};

		//TODO: better way to handle this?
		TArray<FWeakConstraintPair> WeakSinglePointConstraints;
		TMap<FUniqueIdx, FPBDSolveCache> ParticleToCachedSolve;
		int32 ParticleCacheAllocationSize = 0;
	};

} // namespace Chaos
