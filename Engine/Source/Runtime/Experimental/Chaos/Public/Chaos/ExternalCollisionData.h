// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Chaos/Box.h"
#include "Chaos/ParticleHandleFwd.h"
#include "Chaos/PhysicalMaterials.h"
#include "Chaos/Vector.h"
//
// NOTE: This file is widely included in Engine code. 
// Avoid including Chaos headers when possible.
//


class UPhysicalMaterial;
class IPhysicsProxyBase;

namespace Chaos
{
	// Event Emitter flag
	enum EventEmitterFlag
	{
		EmptyDispatcher = 0,
		OwnDispatcher = 1,
		GlobalDispatcher = 2,
		BothDispatcher = 3,
	};

	struct FBaseEventFlag
	{
		FBaseEventFlag() : EmitterFlag(OwnDispatcher) {}

		void SetEmitterFlag(bool LocalEmitter, bool GlobalEmitter)
		{
			EmitterFlag = ComputeEmitterFlag(LocalEmitter, GlobalEmitter);
		}

		static EventEmitterFlag ComputeEmitterFlag(bool LocalEmitter, bool GlobalEmitter)
		{
			EventEmitterFlag EmitterFlagOut = EventEmitterFlag::EmptyDispatcher;
			if (LocalEmitter)
			{
				EmitterFlagOut = EventEmitterFlag::OwnDispatcher;
			}
			if (GlobalEmitter)
			{
				EmitterFlagOut = EventEmitterFlag(EmitterFlagOut | EventEmitterFlag::GlobalDispatcher);
			}
			return EmitterFlagOut;
		}

		EventEmitterFlag EmitterFlag;
	};

	/**
	 * Collision event data stored for use by other systems (e.g. Niagara, gameplay events)
	 */
	struct FCollidingData
	{
		FCollidingData()
			: Location(FVec3((FReal)0.0))
			, AccumulatedImpulse(FVec3((FReal)0.0))
			, Normal(FVec3((FReal)0.0))
			, Velocity1(FVec3((FReal)0.0))
			, Velocity2(FVec3((FReal)0.0))
			, DeltaVelocity1(FVec3((FReal)0.0))
			, DeltaVelocity2(FVec3((FReal)0.0))
			, AngularVelocity1(FVec3((FReal)0.0))
			, AngularVelocity2(FVec3((FReal)0.0))
			, Mass1((FReal)0.0)
			, Mass2((FReal)0.0)
			, PenetrationDepth((FReal)0.0)
			, Mat1(FMaterialHandle())
			, Mat2(FMaterialHandle())
			, bProbe(false)
			, Proxy1(nullptr)
			, Proxy2(nullptr)
			, ShapeIndex1(INDEX_NONE)
			, ShapeIndex2(INDEX_NONE)
			, SolverTime((FReal)0.0)
		{}

		FCollidingData(FVec3 InLocation, FVec3 InAccumulatedImpulse, FVec3 InNormal, FVec3 InVelocity1, FVec3 InVelocity2, FVec3 InDeltaVelocity1, FVec3 InDeltaVelocity2
			, FVec3 InAngularVelocity1, FVec3 InAngularVelocity2, FReal InMass1,FReal InMass2,  FReal InPenetrationDepth, IPhysicsProxyBase* InProxy1, IPhysicsProxyBase* InProxy2
			, int32 InShapeIndex1, int32 InShapeIndex2, FReal InSolverTime)
			: Location(InLocation)
			, AccumulatedImpulse(InAccumulatedImpulse)
			, Normal(InNormal)
			, Velocity1(InVelocity1)
			, Velocity2(InVelocity2)
			, DeltaVelocity1(InDeltaVelocity1)
			, DeltaVelocity2(InDeltaVelocity2)
			, AngularVelocity1(InAngularVelocity1)
			, AngularVelocity2(InAngularVelocity2)
			, Mass1(InMass1)
			, Mass2(InMass2)
			, PenetrationDepth(InPenetrationDepth)
			, Mat1(FMaterialHandle())
			, Mat2(FMaterialHandle())
			, bProbe(false)
			, Proxy1(InProxy1)
			, Proxy2(InProxy2)
			, ShapeIndex1(InShapeIndex1)
			, ShapeIndex2(InShapeIndex2)
			, SolverTime(InSolverTime)
		{}

		FVec3 Location;
		FVec3 AccumulatedImpulse;
		FVec3 Normal;
		FVec3 Velocity1;
		FVec3 Velocity2;
		FVec3 DeltaVelocity1;
		FVec3 DeltaVelocity2;
		FVec3 AngularVelocity1;
		FVec3 AngularVelocity2;
		FReal Mass1;
		FReal Mass2;
		FReal PenetrationDepth;
		FMaterialHandle Mat1;
		FMaterialHandle Mat2;
		bool bProbe;

		// The pointers to the proxies should be used with caution on the Game Thread.
		// Ideally we only ever use these as table keys when acquiring related structures.
		// If we genuinely need to dereference the pointers for any reason, test if they are deleted (nullptr) or
		// pending deletion: if a call to FPhysScene_Chaos::GetOwningComponent<UPrimitiveComponent>() returns
		// nullptr, the proxies should not be used.
		// If either Proxy is nullptr, the structure is invalid.
		IPhysicsProxyBase* Proxy1;
		IPhysicsProxyBase* Proxy2;

		int32 ShapeIndex1;
		int32 ShapeIndex2;

		FReal SolverTime;
	};

	/*
	CollisionData used in the subsystems
	*/
	struct FCollidingDataExt
	{
		FCollidingDataExt()
			: Location(FVec3((FReal)0.0))
			, AccumulatedImpulse(FVec3((FReal)0.0))
			, Normal(FVec3((FReal)0.0))
			, Velocity1(FVec3((FReal)0.0))
			, Velocity2(FVec3((FReal)0.0))
			, AngularVelocity1(FVec3((FReal)0.0))
			, AngularVelocity2(FVec3((FReal)0.0))
			, Mass1((FReal)0.0)
			, Mass2((FReal)0.0)
			, BoundingboxVolume((FReal)-1.0)
			, BoundingboxExtentMin((FReal)-1.0)
			, BoundingboxExtentMax((FReal)-1.0)
			, SurfaceType1(-1)
			, SurfaceType2(-1)
		{}

		FCollidingDataExt(
		    FVec3 InLocation, FVec3 InAccumulatedImpulse, FVec3 InNormal, FVec3 InVelocity1, FVec3 InVelocity2
			, FVec3 InAngularVelocity1, FVec3 InAngularVelocity2, FReal InMass1, FReal InMass2 //, FGeometryParticleHandle* InParticle, FGeometryParticleHandle* InLevelset
			, FReal InBoundingboxVolume, FReal InBoundingboxExtentMin, FReal InBoundingboxExtentMax, int32 InSurfaceType1, int32 InSurfaceType2)
			: Location(InLocation)
			, AccumulatedImpulse(InAccumulatedImpulse)
			, Normal(InNormal)
			, Velocity1(InVelocity1)
			, Velocity2(InVelocity2)
			, AngularVelocity1(InAngularVelocity1)
			, AngularVelocity2(InAngularVelocity2)
			, Mass1(InMass1)
			, Mass2(InMass2)
			, BoundingboxVolume(InBoundingboxVolume)
			, BoundingboxExtentMin(InBoundingboxExtentMin)
			, BoundingboxExtentMax(InBoundingboxExtentMax)
			, SurfaceType1(InSurfaceType1)
			, SurfaceType2(InSurfaceType2)
		{}

		FCollidingDataExt(const FCollidingData& InCollisionData)
			: Location(InCollisionData.Location)
			, AccumulatedImpulse(InCollisionData.AccumulatedImpulse)
			, Normal(InCollisionData.Normal)
			, Velocity1(InCollisionData.Velocity1)
			, Velocity2(InCollisionData.Velocity2)
			, AngularVelocity1(InCollisionData.AngularVelocity1)
			, AngularVelocity2(InCollisionData.AngularVelocity2)
			, Mass1(InCollisionData.Mass1)
			, Mass2(InCollisionData.Mass2)
			, BoundingboxVolume((FReal)-1.0)
			, BoundingboxExtentMin((FReal)-1.0)
			, BoundingboxExtentMax((FReal)-1.0)
			, SurfaceType1(-1)
			, SurfaceType2(-1)
		{
		}

		FVec3 Location;
		FVec3 AccumulatedImpulse;
		FVec3 Normal;
		FVec3 Velocity1, Velocity2;
		FVec3 AngularVelocity1, AngularVelocity2;
		FReal Mass1, Mass2;
		FReal BoundingboxVolume;
		FReal BoundingboxExtentMin, BoundingboxExtentMax;
		int32 SurfaceType1, SurfaceType2;
		FName PhysicalMaterialName1, PhysicalMaterialName2;
	};

	/*
	BreakingData passed from the physics solver to subsystems
	*/
	struct FBreakingData : public FBaseEventFlag
	{
		FBreakingData()
			: FBaseEventFlag()
			, Proxy(nullptr)
			, Location(FVec3((FReal)0.0))
			, Orientation(FRotation3::Identity)
			, Velocity(FVec3((FReal)0.0))
			, AngularVelocity(FVec3((FReal)0.0))
			, Mass((FReal)0.0)
			, BoundingBox(FAABB3(FVec3((FReal)0.0), FVec3((FReal)0.0)))
			, TransformGroupIndex(INDEX_NONE)
			, bFromCrumble(false)
		{}

		// The pointer to the proxy should be used with caution on the Game Thread.
		// Ideally we only ever use this as a table key when acquiring related structures.
		// If we genuinely need to dereference the pointer for any reason, test if it is deleted (nullptr) or
		// pending deletion: if a call to FPhysScene_Chaos::GetOwningComponent<UPrimitiveComponent>() returns
		// nullptr, the proxy should not be used.
		IPhysicsProxyBase* Proxy;
		
		FVec3 Location;
		FRotation3 Orientation;
		FVec3 Velocity;
		FVec3 AngularVelocity;
		FReal Mass;
		Chaos::FAABB3 BoundingBox;
		int32 TransformGroupIndex;
		bool bFromCrumble;
	};

	/*
	CrumblingData passed from the physics solver to subsystems
	*/
	struct FCrumblingData : public FBaseEventFlag
	{
		FCrumblingData()
			: FBaseEventFlag()
			, Proxy(nullptr)
			, Location(FVec3::ZeroVector)
			, Orientation(FRotation3::Identity)
			, LinearVelocity(FVec3::ZeroVector)
			, AngularVelocity(FVec3::ZeroVector)
			, Mass((FReal)0.0)
			, LocalBounds(FAABB3(FVec3((FReal)0.0), FVec3((FReal)0.0)))
		{}

		// The pointer to the proxy should be used with caution on the Game Thread.
		// Ideally we only ever use this as a table key when acquiring related structures.
		// If we genuinely need to dereference the pointer for any reason, test if it is deleted (nullptr) or
		// pending deletion: if a call to FPhysScene_Chaos::GetOwningComponent<UPrimitiveComponent>() returns
		// nullptr, the proxy should not be used.
		IPhysicsProxyBase* Proxy;
		
		FVec3 Location;
		FRotation3 Orientation;
		FVec3 LinearVelocity;
		FVec3 AngularVelocity;
		FReal Mass;
		FAABB3 LocalBounds;
		// optional ( see proxy options )
		TArray<int32> Children;
	};
	
	/*
	BreakingData used in the subsystems
	*/
	struct FBreakingDataExt
	{
		FBreakingDataExt()
			: Location(FVec3((FReal)0.0))
			, Velocity(FVec3((FReal)0.0))
			, AngularVelocity(FVec3((FReal)0.0))
			, Mass((FReal)0.0)
			, BoundingboxVolume((FReal)-1.0)
			, BoundingboxExtentMin((FReal)-1.0)
			, BoundingboxExtentMax((FReal)-1.0)
			, SurfaceType(-1)
		{}

		FBreakingDataExt(FVec3 InLocation
			, FVec3 InVelocity
			, FVec3 InAngularVelocity
			, FReal InMass
			, FGeometryParticleHandle* InParticle
			, FReal InBoundingboxVolume
			, FReal InBoundingboxExtentMin
			, FReal InBoundingboxExtentMax
			, int32 InSurfaceType)
			: Location(InLocation)
			, Velocity(InVelocity)
			, AngularVelocity(InAngularVelocity)
			, Mass(InMass)
			, BoundingboxVolume(InBoundingboxVolume)
			, BoundingboxExtentMin(InBoundingboxExtentMin)
			, BoundingboxExtentMax(InBoundingboxExtentMax)
			, SurfaceType(InSurfaceType)
		{}

		FBreakingDataExt(const FBreakingData& InBreakingData)
			: Location(InBreakingData.Location)
			, Velocity(InBreakingData.Velocity)
			, AngularVelocity(InBreakingData.AngularVelocity)
			, Mass(InBreakingData.Mass)
			, BoundingboxVolume((FReal)-1.0)
			, BoundingboxExtentMin((FReal)-1.0)
			, BoundingboxExtentMax((FReal)-1.0)
			, SurfaceType(-1)
		{
		}

		FVec3 Location;
		FVec3 Velocity;
		FVec3 AngularVelocity;
		FReal Mass;
		FReal BoundingboxVolume;
		FReal BoundingboxExtentMin, BoundingboxExtentMax;
		int32 SurfaceType;

		FVector TransformTranslation;
		FQuat TransformRotation;
		FVector TransformScale;

		FBox BoundingBox;

		// Please don't be tempted to add the code below back. Holding onto a UObject pointer without the GC knowing about it is 
		// not a safe thing to do.
		//UPhysicalMaterial* PhysicalMaterialTest;
		FName PhysicalMaterialName;
	};

	/*
	TrailingData passed from the physics solver to subsystems
	*/
	struct FTrailingData: public FBaseEventFlag
	{
		FTrailingData()
			: Location(FVec3((FReal)0.0))
			, Orientation(FRotation3::Identity)
			, Velocity(FVec3((FReal)0.0))
			, AngularVelocity(FVec3((FReal)0.0))
			, Mass((FReal)0.0)
			, Proxy(nullptr)
			, BoundingBox(FAABB3(FVec3((FReal)0.0), FVec3((FReal)0.0)))
			, TransformGroupIndex(INDEX_NONE)
		{}

		UE_DEPRECATED(5.6, "Deprecated constructor")
		FTrailingData(FVec3 InLocation, FVec3 InVelocity, FVec3 InAngularVelocity, FReal InMass
			, IPhysicsProxyBase* InProxy, Chaos::TAABB<FReal, 3>& InBoundingBox)
			: Location(InLocation)
			, Velocity(InVelocity)
			, AngularVelocity(InAngularVelocity)
			, Mass(InMass)
			, Proxy(InProxy)
			, BoundingBox(InBoundingBox)
			, TransformGroupIndex(INDEX_NONE)
		{}

		FVec3 Location;
		FRotation3 Orientation;
		FVec3 Velocity;
		FVec3 AngularVelocity;
		FReal Mass;
		
		// The pointer to the proxy should be used with caution on the Game Thread.
		// Ideally we only ever use this as a table key when acquiring related structures.
		// If we genuinely need to dereference the pointer for any reason, test if it is deleted (nullptr) or
		// pending deletion: if a call to FPhysScene_Chaos::GetOwningComponent<UPrimitiveComponent>() returns
		// nullptr, the proxy should not be used.
		IPhysicsProxyBase* Proxy;

		Chaos::FAABB3 BoundingBox;
		int32 TransformGroupIndex;
	};

	/*
	TrailingData used in subsystems
	*/
	struct FTrailingDataExt
	{
		FTrailingDataExt()
			: Location(FVec3((FReal)0.0))
			, Velocity(FVec3((FReal)0.0))
			, AngularVelocity(FVec3((FReal)0.0))
			, Mass((FReal)0.0)
			, BoundingboxVolume((FReal)-1.0)
			, BoundingboxExtentMin((FReal)-1.0)
			, BoundingboxExtentMax((FReal)-1.0)
			, SurfaceType(-1)
		{}

		FTrailingDataExt(FVec3 InLocation
			, FVec3 InVelocity
			, FVec3 InAngularVelocity
			, FReal InMass
			, FGeometryParticleHandle* InParticle
			, FReal InBoundingboxVolume
			, FReal InBoundingboxExtentMin
			, FReal InBoundingboxExtentMax
			, int32 InSurfaceType)
			: Location(InLocation)
			, Velocity(InVelocity)
			, AngularVelocity(InAngularVelocity)
			, Mass(InMass)
			, BoundingboxVolume(InBoundingboxVolume)
			, BoundingboxExtentMin(InBoundingboxExtentMin)
			, BoundingboxExtentMax(InBoundingboxExtentMax)
			, SurfaceType(InSurfaceType)
		{}

		FTrailingDataExt(const FTrailingData& InTrailingData)
			: Location(InTrailingData.Location)
			, Velocity(InTrailingData.Velocity)
			, AngularVelocity(InTrailingData.AngularVelocity)
			, Mass(InTrailingData.Mass)
			, BoundingboxVolume((FReal)-1.0)
			, BoundingboxExtentMin((FReal)-1.0)
			, BoundingboxExtentMax((FReal)-1.0)
			, SurfaceType(-1)
		{}

		FVec3 Location;
		FVec3 Velocity;
		FVec3 AngularVelocity;
		FReal Mass;
		FReal BoundingboxVolume;
		FReal BoundingboxExtentMin, BoundingboxExtentMax;
		int32 SurfaceType;
	};

	struct FSleepingData
	{
		FSleepingData()
			: Proxy(nullptr)
			, Sleeping(true)
		{}

		FSleepingData(IPhysicsProxyBase* InProxy, bool InSleeping)
			: Proxy(InProxy)
			, Sleeping(InSleeping)
		{}

		// The pointer to the proxy should be used with caution on the Game Thread.
		// Ideally we only ever use this as a table key when acquiring related structures.
		// If we genuinely need to dereference the pointer for any reason, test if it is deleted (nullptr) or
		// pending deletion: if a call to FPhysScene_Chaos::GetOwningComponent<UPrimitiveComponent>() returns
		// nullptr, the proxy should not be used.
		IPhysicsProxyBase* Proxy;

		bool Sleeping;	// if !Sleeping == Awake
	};

	/*
	RemovalData passed from the physics solver to subsystems
	*/
	struct FRemovalData
	{
		FRemovalData()
			: Location(FVec3((FReal)0.0))
			, Mass((FReal)0.0)
			, Proxy(nullptr)
			, BoundingBox(FAABB3(FVec3((FReal)0.0), FVec3((FReal)0.0)))
		{}

		FRemovalData(FVec3 InLocation, FReal InMass, IPhysicsProxyBase* InProxy, Chaos::TAABB<FReal, 3>& InBoundingBox)
			: Location(InLocation)
			, Mass(InMass)
			, Proxy(InProxy)
			, BoundingBox(InBoundingBox)
		{}

		FVec3 Location;
		FReal Mass;

		// The pointer to the proxy should be used with caution on the Game Thread.
		// Ideally we only ever use this as a table key when acquiring related structures.
		// If we genuinely need to dereference the pointer for any reason, test if it is deleted (nullptr) or
		// pending deletion: if a call to FPhysScene_Chaos::GetOwningComponent<UPrimitiveComponent>() returns
		// nullptr, the proxy should not be used.
		IPhysicsProxyBase* Proxy;

		Chaos::FAABB3 BoundingBox;
	};

	/*
	RemovalData used in subsystems
	*/
	struct FRemovalDataExt
	{
		FRemovalDataExt()
			: Location(FVec3((FReal)0.0))
			, Mass((FReal)0.0)
			, BoundingboxVolume((FReal)-1.0)
			, BoundingboxExtentMin((FReal)-1.0)
			, BoundingboxExtentMax((FReal)-1.0)
			, SurfaceType(-1)
		{}

		FRemovalDataExt(FVec3 InLocation
			, FReal InMass
			, FGeometryParticleHandle* InParticle
			, FReal InBoundingboxVolume
			, FReal InBoundingboxExtentMin
			, FReal InBoundingboxExtentMax
			, int32 InSurfaceType)
			: Location(InLocation)
			, Mass(InMass)
			, BoundingboxVolume(InBoundingboxVolume)
			, BoundingboxExtentMin(InBoundingboxExtentMin)
			, BoundingboxExtentMax(InBoundingboxExtentMax)
			, SurfaceType(InSurfaceType)
		{}

		FRemovalDataExt(const FRemovalData& InRemovalData)
			: Location(InRemovalData.Location)
			, Mass(InRemovalData.Mass)
			, BoundingboxVolume((FReal)-1.0)
			, BoundingboxExtentMin((FReal)-1.0)
			, BoundingboxExtentMax((FReal)-1.0)
			, SurfaceType(-1)
		{}

		FVec3 Location;
		FReal Mass;
		FReal BoundingboxVolume;
		FReal BoundingboxExtentMin, BoundingboxExtentMax;
		int32 SurfaceType;
	};

	template<class T, int d>
	using TCollisionData UE_DEPRECATED(4.27, "Deprecated. this class is to be deleted, use FCollidingData instead") = FCollidingData;

	template<class T, int d>
	using TCollisionDataExt UE_DEPRECATED(4.27, "Deprecated. this class is to be deleted, use FCollidingDataExt instead") = FCollidingDataExt;

	template<class T, int d>
	using TBreakingData UE_DEPRECATED(4.27, "Deprecated. this class is to be deleted, use FBreakingData instead") = FBreakingData;

	template<class T, int d>
	using TBreakingDataExt UE_DEPRECATED(4.27, "Deprecated. this class is to be deleted, use FBreakingDataExt instead") = FBreakingDataExt;

	template<class T, int d>
	using TTrailingData UE_DEPRECATED(4.27, "Deprecated. this class is to be deleted, use FTrailingData instead") = FTrailingData;

	template<class T, int d>
	using TTrailingDataExt UE_DEPRECATED(4.27, "Deprecated. this class is to be deleted, use FTrailingDataExt instead") = FTrailingDataExt;

	template<class T, int d>
	using TSleepingData UE_DEPRECATED(4.27, "Deprecated. this class is to be deleted, use FSleepingData instead") = FSleepingData;
}


