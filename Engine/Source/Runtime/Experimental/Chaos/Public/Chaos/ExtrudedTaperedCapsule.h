// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/ImplicitObject.h"
#include "Chaos/Core.h"
#include "Math/UnrealMath.h"

namespace Chaos
{
	class FExtrudedTaperedCapsule: public FImplicitObject
	{
	public:
		FExtrudedTaperedCapsule()
		    : FImplicitObject(EImplicitObject::FiniteConvex, ImplicitObjectType::ExtrudedTaperedCapsule)
		{
		}
		FExtrudedTaperedCapsule(const FVec3f& InOrigin, const FVec3f& InMajorAxis, const FVec3f& InMinorAxis, const FRealSingle InHalfLength, const FRealSingle InHalfWidth, const FRealSingle InRadius1, const FRealSingle InRadius2)
		    : FImplicitObject(EImplicitObject::FiniteConvex, ImplicitObjectType::ExtrudedTaperedCapsule)
		    , Origin(InOrigin)
			, MajorAxis(InMajorAxis.GetSafeNormal())
			, MinorAxis((InMinorAxis -InMinorAxis.Dot(MajorAxis)*MajorAxis).GetSafeNormal())
			, HalfLength(InHalfLength)
			, HalfWidth(InHalfWidth)
		    , Radius1(InRadius1)
		    , Radius2(InRadius2)
			, LocalBoundingBox(Origin,Origin)
		{
			LocalBoundingBox.GrowToInclude(Origin + MajorAxis * HalfLength + MinorAxis * HalfWidth);
			LocalBoundingBox.GrowToInclude(Origin + MajorAxis * HalfLength - MinorAxis * HalfWidth);
			LocalBoundingBox.GrowToInclude(Origin - MajorAxis * HalfLength + MinorAxis * HalfWidth);
			LocalBoundingBox.GrowToInclude(Origin - MajorAxis * HalfLength - MinorAxis * HalfWidth);
			const FRealSingle MaxRadius = FMath::Max(Radius1, Radius2);
			LocalBoundingBox = FAABB3(LocalBoundingBox.Min() - FVec3(MaxRadius), LocalBoundingBox.Max() + FVec3(MaxRadius));
		}
		FExtrudedTaperedCapsule(const FExtrudedTaperedCapsule& Other)
		    : FImplicitObject(EImplicitObject::FiniteConvex, ImplicitObjectType::ExtrudedTaperedCapsule)
		    , Origin(Other.Origin)
			, MajorAxis(Other.MajorAxis)
			, MinorAxis(Other.MinorAxis)
			, HalfLength(Other.HalfLength)
			, HalfWidth(Other.HalfWidth)
			, Radius1(Other.Radius1)
			, Radius2(Other.Radius2)
			, LocalBoundingBox(Other.LocalBoundingBox)
		{
		}
		FExtrudedTaperedCapsule(FExtrudedTaperedCapsule&& Other)
		    : FImplicitObject(EImplicitObject::FiniteConvex, ImplicitObjectType::ExtrudedTaperedCapsule)
		    , Origin(MoveTemp(Other.Origin))
			, MajorAxis(MoveTemp(Other.MajorAxis))
			, MinorAxis(MoveTemp(Other.MinorAxis))
			, HalfLength(Other.HalfLength)
			, HalfWidth(Other.HalfWidth)
			, Radius1(Other.Radius1)
			, Radius2(Other.Radius2)
			, LocalBoundingBox(MoveTemp(Other.LocalBoundingBox))
		{
		}

		~FExtrudedTaperedCapsule() {}

		static constexpr EImplicitObjectType StaticType() { return ImplicitObjectType::ExtrudedTaperedCapsule; }

		virtual const FAABB3 BoundingBox() const override { return LocalBoundingBox; }

		FReal PhiWithNormal(const FVec3& x, FVec3& OutNormal) const
		{
			const FVec3f OriginToX = x - Origin;
			const FRealSingle DistanceAlongMajorAxis = FMath::Clamp(FVec3f::DotProduct(OriginToX, MajorAxis), -HalfLength, HalfLength);
			const FRealSingle DistanceAlongMinorAxis = FMath::Clamp(FVec3f::DotProduct(OriginToX, MinorAxis), -HalfWidth, HalfWidth);
			const FVec3f ClosestPoint = Origin + MajorAxis * DistanceAlongMajorAxis  + MinorAxis * DistanceAlongMinorAxis;
			const FRealSingle Radius = (HalfLength > UE_SMALL_NUMBER) ? FMath::Lerp(Radius2, Radius1, .5f*(DistanceAlongMajorAxis / HalfLength) + .5f) : FMath::Max(Radius1, Radius2);
			OutNormal = (x - ClosestPoint);
			return OutNormal.SafeNormalize() - Radius;
		}

		/* Radius at +MajorAxis end*/
		FRealSingle GetRadius1() const { return Radius1; }
		/* Radius at -MajorAxis end*/
		FRealSingle GetRadius2() const { return Radius2; }
		FRealSingle GetHalfLength() const { return HalfLength; }
		FRealSingle GetHalfWidth() const { return HalfWidth; }
		/** Returns the center of the rectangle */
		const FVec3f& GetOrigin() const 
		{
			return Origin;
		}
		const FVec3f& GetOriginf() const { return Origin; }
		/** Length axis */
		const FVec3f& GetMajorAxis() const { return MajorAxis; }
		/** Width axis */
		const FVec3f& GetMinorAxis() const { return MinorAxis; }

		virtual uint32 GetTypeHash() const override
		{
			uint32 Result = UE::Math::GetTypeHash(Origin);
			Result = HashCombine(Result, UE::Math::GetTypeHash(MajorAxis));
			Result = HashCombine(Result, UE::Math::GetTypeHash(MinorAxis));
			Result = HashCombine(Result, ::GetTypeHash(HalfLength));
			Result = HashCombine(Result, ::GetTypeHash(HalfWidth));
			Result = HashCombine(Result, ::GetTypeHash(Radius1));
			Result = HashCombine(Result, ::GetTypeHash(Radius2));
			return Result;
		}

#if INTEL_ISPC
		// See PerParticlePBDCollisionConstraint.cpp
		// ISPC code has matching structs for interpreting FImplicitObjects.
		// This is used to verify that the structs stay the same.
		struct FISPCDataVerifier
		{
			static constexpr int32 OffsetOfOrigin() { return offsetof(FExtrudedTaperedCapsule, Origin); }
			static constexpr int32 SizeOfOrigin() { return sizeof(FExtrudedTaperedCapsule::Origin); }
			static constexpr int32 OffsetOfMajorAxis() { return offsetof(FExtrudedTaperedCapsule, MajorAxis); }
			static constexpr int32 SizeOfMajorAxis() { return sizeof(FExtrudedTaperedCapsule::MajorAxis); }
			static constexpr int32 OffsetOfMinorAxis() { return offsetof(FExtrudedTaperedCapsule, MinorAxis); }
			static constexpr int32 SizeOfMinorAxis() { return sizeof(FExtrudedTaperedCapsule::MinorAxis); }
			static constexpr int32 OffsetOfHalfLength() { return offsetof(FExtrudedTaperedCapsule, HalfLength); }
			static constexpr int32 SizeOfHalfLength() { return sizeof(FExtrudedTaperedCapsule::HalfLength); }
			static constexpr int32 OffsetOfHalfWidth() { return offsetof(FExtrudedTaperedCapsule, HalfWidth); }
			static constexpr int32 SizeOfHalfWidth() { return sizeof(FExtrudedTaperedCapsule::HalfWidth); }
			static constexpr int32 OffsetOfRadius1() { return offsetof(FExtrudedTaperedCapsule, Radius1); }
			static constexpr int32 SizeOfRadius1() { return sizeof(FExtrudedTaperedCapsule::Radius1); }
			static constexpr int32 OffsetOfRadius2() { return offsetof(FExtrudedTaperedCapsule, Radius2); }
			static constexpr int32 SizeOfRadius2() { return sizeof(FExtrudedTaperedCapsule::Radius2); }
		};
		friend FISPCDataVerifier;
#endif // #if INTEL_ISPC

	private:

		FVec3f Origin;
		FVec3f MajorAxis;
		FVec3f MinorAxis;
		FRealSingle HalfLength;
		FRealSingle HalfWidth;
		FRealSingle Radius1;
		FRealSingle Radius2;
		FAABB3 LocalBoundingBox;
	};


} // namespace Chaos
