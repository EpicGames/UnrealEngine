// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/ArrayND.h"
#include "Chaos/Core.h"
#include "Chaos/Complex.h"
#include "Chaos/UniformGrid.h"

namespace Chaos
{
class FFFT3
{
public:
	using FUniformGrid = TUniformGrid<FReal, 3>;
	using FArrayNDOfComplex = TArrayND<FComplex, 3>;

public:
	static void Transform(const FUniformGrid& Grid, const TArrayND<FVec3, 3>& Velocity, FArrayNDOfComplex& u, FArrayNDOfComplex& v, FArrayNDOfComplex& w);
	static void InverseTransform(const FUniformGrid& Grid, TArrayND<FVec3, 3>& Velocity, const FArrayNDOfComplex& u, const FArrayNDOfComplex& v, const FArrayNDOfComplex& w, const bool Normalize);
	static void MakeDivergenceFree(const FUniformGrid& Grid, FArrayNDOfComplex& u, FArrayNDOfComplex& v, FArrayNDOfComplex& w);
};
}
