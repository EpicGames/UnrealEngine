// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/ArrayND.h"
#include "Chaos/FFT.h"
#include "Chaos/UniformGrid.h"

namespace Chaos
{
template<int d>
bool IsPowerOfTwo(const TVector<int32, d>& Counts)
{
	for (int32 i = 0; i < d; ++i)
	{
		if (Counts[i] & (Counts[i] - 1))
			return false;
	}
	return true;
}

class FFFTProjection3
{
  public:
	  FFFTProjection3(const int32 NumIterations = 1)
	    : MNumIterations(NumIterations) {}
	~FFFTProjection3() = default;

	void Apply(const TUniformGrid<FReal, 3>& Grid, TArrayND<FVec3, 3>& Velocity, const TArrayND<bool, 3>& BoundaryConditions, const FReal dt)
	{
		check(IsPowerOfTwo(Grid.Counts()));
		int32 size = Grid.Counts().Product();
		TVec3<int32> Counts = Grid.Counts();
		Counts[2] = Counts[2] / 2 + 1;
		TArrayND<FComplex, 3> u(Counts), v(Counts), w(Counts);
		TArrayND<FVec3, 3> VelocitySaved = Velocity.Copy();
		for (int32 iteration = 0; iteration < MNumIterations; ++iteration)
		{
			FFFT3::Transform(Grid, Velocity, u, v, w);
			FFFT3::MakeDivergenceFree(Grid, u, v, w);
			FFFT3::InverseTransform(Grid, Velocity, u, v, w, true);
			for (int32 i = 0; i < size; ++i)
			{
				Velocity[i] = BoundaryConditions[i] ? VelocitySaved[i] : Velocity[i];
			}
		}
	}

  private:
	int32 MNumIterations;
};
}
