// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Chaos/Core.h"
#include "Containers/Array.h"

namespace Chaos
{
	/**
	* Move the elements in the array Items in the range [BeginIndex, EndIndex) down by DownShift. Does not resize the array.
	*/
	template<typename TItemArray>
	void MoveArrayItemsDown(TItemArray& Items, const int32 BeginIndex, const int32 EndIndex, const int32 DownShift)
	{
		check(DownShift >= 0);
		check(BeginIndex - DownShift >= 0);
		check(EndIndex >= BeginIndex);
		check(EndIndex <= Items.Num());

		if (DownShift > 0)
		{
			for (int32 SrcIndex = BeginIndex; SrcIndex < EndIndex; ++SrcIndex)
			{
				Items[SrcIndex - DownShift] = MoveTemp(Items[SrcIndex]);
			}
		}
	}

	/**
	* Remove a set of items from an array
	* @tparam TItemArray An array of items of any movable type
	* @param Items an array of items to remove elements from
	* @param SortedIndicesToRemove the set of array indices to be removed. Must be sorted low to high. May contain duplicates.
	*/
	template<typename TItemArray>
	void RemoveArrayItemsAtSortedIndices(TItemArray& Items, const TArrayView<const int32>& SortedIndicesToRemove)
	{
		if (SortedIndicesToRemove.IsEmpty())
		{
			return;
		}

		int32 DestShift = 1;
		int32 IndexToRemove0 = SortedIndicesToRemove[0];
		for (int32 IndicesIndex = 1; IndicesIndex < SortedIndicesToRemove.Num(); ++IndicesIndex)
		{
			const int32 IndexToRemove1 = SortedIndicesToRemove[IndicesIndex];

			if (IndexToRemove1 != IndexToRemove0)
			{
				MoveArrayItemsDown(Items, IndexToRemove0 + 1, IndexToRemove1, DestShift);

				IndexToRemove0 = IndexToRemove1;
				DestShift++;
			}
		}

		MoveArrayItemsDown(Items, IndexToRemove0 + 1, Items.Num(), DestShift);

		Items.SetNum(Items.Num() - DestShift, EAllowShrinking::No);
	}

}