// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Internal

namespace Chaos
{
	/**
	 * Container type for double buffered physics data. Wrap whatever results object
	 * in this to have well definied semantics for accessing each side of a buffer and
	 * flipping it
	 */
	template<typename DataType>
	class TBufferedData
	{
	public:

		TBufferedData()
			: SyncCounter(0)
			, BufferIndex(0)
		{
			DataSyncCounts[0] = 0;
			DataSyncCounts[1] = 0;
		}

		/**
		 * Flips the double buffer, no locks here - if synchronizing multiple threads make sure there's a lock somewhere
		 */
		void Flip()
		{
			BufferIndex.Store(GetGameDataIndex());
		}

		/**
		 * Get a readable reference for the game thread side of the double buffer
		 */
		const DataType& GetGameDataForRead() const
		{
			return Data[GetGameDataIndex()];
		}

		/**
		 * Get a readable reference for the physics side of the double buffer
		 */
		const DataType& GetPhysicsDataForRead() const
		{
			return Data[GetPhysicsDataIndex()];
		}

		/**
		 * Get the counter for the last written state on the game side
		 */
		int32 GetGameDataSyncCount() const 
		{
			return DataSyncCounts[GetGameDataIndex()];
		}

		/**
		 * Get the counter for the last written state on the physics side
		 */
		int32 GetPhysicsDataSyncCount() const
		{
			return DataSyncCounts[GetPhysicsDataIndex()];
		}

		/**
		 * Only for the game side to call, gets a writable reference to the game side data.
		 * Mainly useful for exchanging ptrs in the data type. For copying just call GetGameDataForRead
		 */
		DataType& GetGameDataForWrite()
		{
			return Data[GetGameDataIndex()];
		}

		/**
		 * Only for the physics side to call, gets a writable reference to the physics side and
		 * increments the current sync counter to uniquely identify this write
		 */
		DataType& GetPhysicsDataForWrite()
		{
			int32 DataIndex = GetPhysicsDataIndex();
			DataSyncCounts[DataIndex] = ++SyncCounter;
			return Data[DataIndex];
		}

		/**
		 * Direct access to buffered data, useful to initialise members before
		 * beginning simulation. Never use once the data is being managed over
		 * multiple threads
		 */
		DataType& Get(int32 InIndex)
		{
			checkSlow(InIndex == 0 || InIndex == 1);
			return Data[InIndex];
		}

	private:

		int32 GetPhysicsDataIndex() const
		{
			return BufferIndex.Load();
		}

		int32 GetGameDataIndex() const
		{
			return GetPhysicsDataIndex() == 1 ? 0 : 1;
		}

		// Counter used to identify writes
		uint32 SyncCounter;
		// Counter values for each side of the buffer
		uint32 DataSyncCounts[2];

		// Atomic index for accessing the buffer sides
		TAtomic<int32> BufferIndex;

		// The actual data type stored
		DataType Data[2];
	};

	/**
	 * Lockable resource for use with chaos data.
	 * External threads should call GetRead and ReleaseRead on this object to lock it for read.
	 * When this is done the physics thread will not be allowed to swap the buffers so external
	 * reads are safe.
	 * Physics code can use GetWritable to get access to the write buffer, no locks are performed here
	 * as the lock is only required from the physics side when swapping the buffers. No external thread
	 * should ever attempt to swap the buffer. Only the owning thread
	 */
	template<typename ResourceType>
	class TChaosReadWriteResource
	{
	public:

		template<typename... ArgTypes>
		TChaosReadWriteResource(EInPlace, ArgTypes&&... Args)
			: ResourceLock()
			, Buffer{{Args...},{Forward<ArgTypes>(Args)...}}
			, ReadIndex(0)
		{}

		/**
		 * Swaps the read/write sides of the buffer safely so no readers
		 * are interrupted (grabs a write lock)
		 */
		void Swap()
		{
			ResourceLock.WriteLock();

			ReadIndex = (ReadIndex + 1) % 2;

			ResourceLock.WriteUnlock();
		}

		/**  Gets the readable version of the resource */
		const ResourceType& GetRead() const
		{
			ResourceLock.ReadLock();
			return Buffer[ReadIndex];
		}

		/** Release read locks */
		void ReleaseRead() const
		{
			ResourceLock.ReadUnlock();
		}

		/** Gets the owning thread's writable side of the buffer */
		ResourceType& GetWritable()
		{
			return Buffer[(ReadIndex + 1) % 2];
		}
		/** Gets the owning thread's const readable side of the buffer */
		const ResourceType& GetReadable() const
		{
			return Buffer[(ReadIndex + 1) % 2];
		}

	private:

		/** Mutable lock so a const resource can be read locked, keeps this object logically if not literally const */
		mutable FRWLock ResourceLock;

		/** Storage for the buffered objects */
		ResourceType Buffer[2];

		/** Index into the buffer for which side is the external side */
		int32 ReadIndex;
	};
}
