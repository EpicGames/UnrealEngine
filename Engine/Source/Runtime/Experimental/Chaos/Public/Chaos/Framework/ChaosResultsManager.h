// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/PullPhysicsDataImp.h"

namespace Chaos
{
	class FChaosMarshallingManager;
	
	extern FRealSingle SecondChannelDelay;
	extern int32 DefaultNumActiveChannels;

	struct FChaosRigidInterpolationData
	{
		FDirtyRigidParticleData Prev;
		FDirtyRigidParticleData Next;
	};

	struct FChaosGeometryCollectionInterpolationData
	{
		FDirtyGeometryCollectionData Prev;
		FDirtyGeometryCollectionData Next;
	};

	struct FChaosClusterUnionInterpolationData
	{
		FDirtyClusterUnionData Prev;
		FDirtyClusterUnionData Next;
	};
	
	struct FChaosInterpolationResults
	{
		FChaosInterpolationResults()
			: Prev(nullptr)
			, Next(nullptr)
		{
		}

		FChaosInterpolationResults(const FChaosInterpolationResults& Other) = delete;
		FChaosInterpolationResults(FChaosInterpolationResults&& Other)
			: RigidInterpolations(MoveTemp(Other.RigidInterpolations))
			, GeometryCollectionInterpolations(MoveTemp(Other.GeometryCollectionInterpolations))
			, ClusterUnionInterpolations(MoveTemp(Other.ClusterUnionInterpolations))
			, Prev(Other.Prev)
			, Next(Other.Next)
			, Alpha(Other.Alpha)
		{
			Other.Prev = nullptr;
			Other.Next = nullptr;
		}

		CHAOS_API void Reset();
		
		TArray<FChaosRigidInterpolationData> RigidInterpolations;
		TArray<FChaosGeometryCollectionInterpolationData> GeometryCollectionInterpolations;
		TArray<FChaosClusterUnionInterpolationData> ClusterUnionInterpolations;
		FPullPhysicsData* Prev;
		FPullPhysicsData* Next;
		FRealSingle Alpha;
	};

	struct FChaosResultsChannel;

	class FChaosResultsManager
	{
	public:
		CHAOS_API FChaosResultsManager(FChaosMarshallingManager& InMarshallingManager);
		FChaosResultsManager(const FChaosResultsManager& Other) = delete;
		FChaosResultsManager(FChaosResultsManager&& Other) = default;
		
		CHAOS_API ~FChaosResultsManager();

		CHAOS_API const FChaosInterpolationResults& PullSyncPhysicsResults_External();
		CHAOS_API TArray<const FChaosInterpolationResults*> PullAsyncPhysicsResults_External(const FReal ResultsTime);

		CHAOS_API void RemoveProxy_External(FSingleParticlePhysicsProxy* Proxy);

	private:

		friend FChaosResultsChannel;

		CHAOS_API FPullPhysicsData* PopPullData_External(int32 ChannelIdx);
		CHAOS_API void FreePullData_External(FPullPhysicsData* PullData, int32 ChannelIdx);

		TArray<FChaosResultsChannel*> Channels;
		FChaosMarshallingManager& MarshallingManager;

		static constexpr int32 MaxNumChannels = 2;

		struct FPullDataQueueInfo
		{
			FPullDataQueueInfo(FPullPhysicsData* Data, int32 NumActiveChannels)
			: PullData(Data)
			{
				for(int32 Idx = 0; Idx < NumActiveChannels; ++Idx)
				{
					bHasPopped[Idx] = false;
					bPendingFree[Idx] = false;
				}

				//inactive channels are treated like they've already consumed the data
				for (int32 Idx = NumActiveChannels; Idx < MaxNumChannels; ++Idx)
				{
					bHasPopped[Idx] = true;
					bPendingFree[Idx] = true;
				}
			}

			FPullDataQueueInfo(FPullDataQueueInfo& Other) = delete;
			FPullDataQueueInfo(FPullDataQueueInfo&& Other) = default;

			FPullPhysicsData* PullData;
			bool bHasPopped[MaxNumChannels];
			bool bPendingFree[MaxNumChannels];
		};

		TArray<FPullDataQueueInfo> InternalQueue;
		int32 NumActiveChannels;
		TArray<FRealSingle> PerChannelTimeDelay;
	};
}
