// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/ChunkedArray.h"

namespace Chaos
{
	namespace Private
	{
		template<typename InElementType, uint32 TargetBytesPerChunk = 16384, typename AllocatorType = FDefaultAllocator, typename = TEnableIf<std::is_trivially_destructible_v<InElementType>>>
		class TChaosChunkedArray : public TChunkedArray<InElementType, TargetBytesPerChunk, AllocatorType>
		{
		public:
			// This function will set the number of element to 0, but won't allocate more memory neither won't free memory,
			// and won't call any destructor.
			void Reset()
			{
				this->NumElements = 0;
			}
		};

	}
}