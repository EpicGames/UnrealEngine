// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"

namespace Chaos
{
	 /**
	 * A handle to an item held by index in a container.
	 * The container must provide the following API:
	 * 
	 * 	-	int NumItems() const;
	 *	-	bool IsValid(int32 Index) const;
	 *	-	FItem* TryGetItem(int32 Index);
	 *	-	const FItem* TryGetItem(int32 Index) const;
	 *	-	FItem& GetItem(int32 Index);
	 *	-	const FItem& GetItem(int32 Index) const;
	 */   
	template<typename T_CONTAINER, typename T_ITEM>
	class TConstContainerItemHandle
	{
	public:
		using FContainer = T_CONTAINER;
		using FItem = T_ITEM;
		using FHandle = TConstContainerItemHandle<FContainer, FItem>;

		TConstContainerItemHandle()
			: Container(nullptr)
			, Index(INDEX_NONE)
		{
		}

		TConstContainerItemHandle(const T_CONTAINER& InContainer, int InIndex)
			: Container(&InContainer)
			, Index(InIndex)
		{
		}

		inline const FItem* TryGet() const
		{
			if (Container != nullptr)
			{
				return Container->TryGetItem(Index);
			}
			return nullptr;
		}

		inline const FItem& Get() const
		{
			check(Container != nullptr);
			return Container->GetItem(Index);
		}

		inline const FItem* operator->() const
		{
			return &Get();
		}

		inline const FItem& operator*() const
		{
			return Get();
		}

		inline bool IsValid() const
		{
			return (Container != nullptr) && Container->IsValid(Index);
		}

		inline void Reset()
		{
			Container = nullptr;
			Index = INDEX_NONE;
		}

		inline const FContainer* GetContainer() const
		{
			return Container;
		}

		inline int GetIndex() const
		{
			return Index;
		}

		friend inline bool operator==(const FHandle& L, const FHandle& R)
		{
			return (L.GetContainer() == R.GetContainer()) && (L.GetIndex() == R.GetIndex());
		}

		friend inline bool operator!=(const FHandle& L, const FHandle& R)
		{
			return !(L == R);
		}

		friend inline bool operator<(const FHandle& L, const FHandle& R)
		{
			check(L.IsValid());
			check(R.IsValid());
			return L.GetIndex() < R.GetIndex();
		}

	private:
		const FContainer* Container;
		int Index;
	};

	/**
	 * Same as TConstContainerItemHandle but for non-const containers/items
	 */
	template<typename T_CONTAINER, typename T_ITEM>
	class TContainerItemHandle : private TConstContainerItemHandle<T_CONTAINER, T_ITEM>
	{
	public:
		using Base = TConstContainerItemHandle<T_CONTAINER, T_ITEM>;
		using FContainer = T_CONTAINER;
		using FItem = T_ITEM;
		using FHandle = TContainerItemHandle<FContainer, FItem>;

		TContainerItemHandle()
		{
		}

		TContainerItemHandle(T_CONTAINER& InContainer, int InIndex)
			: Base(InContainer, InIndex)
		{
		}

		// Auto-cast to const handle reference
		inline operator const TConstContainerItemHandle<T_CONTAINER, T_ITEM>& () const
		{
			return *reinterpret_cast<const TConstContainerItemHandle<T_CONTAINER, T_ITEM>*>(this);
		}

		inline FItem* TryGet() const
		{
			return const_cast<FItem*>(Base::TryGet());
		}

		inline FItem& Get() const
		{
			return const_cast<FItem&>(Base::Get());
		}

		inline FItem* operator->() const
		{
			return &const_cast<FItem&>(Base::Get());
		}

		inline FItem& operator*() const
		{
			return const_cast<FItem&>(Base::Get());
		}

		using Base::IsValid;

		using Base::Reset;

		inline FContainer* GetContainer() const
		{
			return const_cast<FContainer*>(Base::GetContainer());
		}

		using Base::GetIndex;

		friend inline bool operator==(const FHandle& L, const FHandle& R)
		{
			return (L.GetContainer() == R.GetContainer()) && (L.GetIndex() == R.GetIndex());
		}

		friend inline bool operator!=(const FHandle& L, const FHandle& R)
		{
			return !(L == R);
		}

		friend inline bool operator<(const FHandle& L, const FHandle& R)
		{
			check(L.IsValid());
			check(R.IsValid());
			return L.GetIndex() < R.GetIndex();
		}
	};


}
