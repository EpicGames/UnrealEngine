// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Build.h"

#ifndef CHAOS_DEBUG_SUBSTEP
#define CHAOS_DEBUG_SUBSTEP !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
#endif

#if CHAOS_DEBUG_SUBSTEP

#include "HAL/CriticalSection.h"
#include "HAL/ThreadSafeBool.h"
#include "Containers/Queue.h"

class FEvent;

namespace Chaos
{
	/** Debug only class used to control the pausing/stepping/substepping of a debug solver thread. */
	class FDebugSubstep final
	{
		friend class FDebugSolverTask;
		friend class FDebugSolverTasks;

	public:
		FDebugSubstep();
		CHAOS_API ~FDebugSubstep();

		/** Return whether debugging mode/pausing to substeps is enabled. */
		bool IsEnabled() const { return bIsEnabled; }

		/**
		 * Add a new potential pause point where the debug solver thread can wait until the next step/substep command.
		 * @param Label The reference (if any) that will be used in verbose logs when this point is reached, or nullptr otherwise.
		 * Only call from the solver thread.
		 * It will fail if called from inside a parallel for loop, or any other thread.
		 */
		FORCEINLINE void Add(const TCHAR* Label = nullptr) const { Add(false, Label); }

		/** Enable/disable substep pause points. */
		CHAOS_API void Enable(bool bEnable);

		/** Allow progress to the next substep (only works once this object is enabled). */
		CHAOS_API void ProgressToSubstep();

		/** Allow progress to the next step (only works once this object is enabled). */
		CHAOS_API void ProgressToStep();

	private:
		FDebugSubstep(const FDebugSubstep&) = delete;
		FDebugSubstep& operator=(const FDebugSubstep&) = delete;

		/** Initialize sync events. */
		CHAOS_API void Initialize();

		/** Shutdown and release sync events. Once released the debug thread cannot be restarted unless Initialize is called first. */
		CHAOS_API void Release();

		/** Return true when initialized. */
		FORCEINLINE bool IsInitialized() const { return !!ProgressEvent && !!SubstepEvent; }

		/*
		 * Disable and wait for the completion of the debug thread.
		 * Not thread safe. Must be called from within the physics thread or with the physics thread locked.
		 * Once shutdown, the debug thread can still be restarted when SyncAdvance is called.
		 */
		CHAOS_API void Shutdown();

		/** Set the id of the thread the debug substepping will be running in. */
		CHAOS_API void AssumeThisThread();

		/**
		 * Control substepping progress.
		 * Start substepping, wait until the next substep is reached, or return straightaway if debugging is disabled.
		 * @param bIsSolverEnabled The state of the solver. The substepping will stop when the solver is disabled and delay starting again until re-enabled.
		 * @return Whether the debug thread needs running.
		 */
		CHAOS_API bool SyncAdvance(bool bIsSolverEnabled);

		/**
		 * Add a new step or substep.
		 * @param bInStep Add a step instead of a substep when this is true (for internal use only in the solver debug thread loop).
		 * @param Label The reference (if any) that will be used in verbose logs when this point is reached, or nullptr otherwise.
		 */
		CHAOS_API void Add(bool bInStep, const TCHAR* Label) const;

		/** Set sync events to start the substepping process. */
		CHAOS_API void Start();

		/** Trigger/wait for sync events to stop the substepping process. */
		CHAOS_API void Stop();

		/** Trigger/wait for sync events to step/substep. */
		CHAOS_API void Substep(bool bShouldStep);

	private:
		enum class ECommand { Enable, Disable, ProgressToSubstep, ProgressToStep };

		FThreadSafeBool bIsEnabled;    // Status of the debugging thread. Can find itself in a race condition while the Add() method is ran outside of the debug thread, hence the FThreadSafeBool.
		TQueue<ECommand, EQueueMode::Mpsc> CommandQueue;  // Command queue, thread safe, Multiple-producers single-consumer (MPSC) model.
		FEvent* ProgressEvent;         // Progress synchronization event.
		FEvent* SubstepEvent;          // Substep synchronization event.
		uint32 ThreadId;               // Thread id used to check that the debug substep code is still running within the debug thread.
		mutable bool bWaitForStep;     // Boolean used to flag the completion of a step. Set within the a const function, hence the mutable.
		bool bShouldEnable;            // Whether an enable command has been requested. The debugging thread might not be ready yet (e.g. disabled solver), this will ensure it still enable the debug substepping once it is ready.
	};
}

#else  // #if CHAOS_DEBUG_SUBSTEP

#include "HAL/Platform.h"

namespace Chaos
{
	/**
	 * Debug substep stub for non debug builds.
	 */
	class FDebugSubstep final
	{
	public:
		FDebugSubstep() {}
		~FDebugSubstep() = default;

		bool IsEnabled() const { return false; }

		void Add(const TCHAR* /*Label*/ = nullptr) {}

		void Enable(bool /*bEnable*/) {}

		void ProgressToSubstep() {}
		void ProgressToStep() {}

	private:
		FDebugSubstep(const FDebugSubstep&) = delete;
		FDebugSubstep& operator=(const FDebugSubstep&) = delete;
	};
}

#endif  // #if CHAOS_DEBUG_SUBSTEP #else
