// Copyright Epic Games, Inc.All Rights Reserved.
#pragma once

#include "Containers/HashTable.h"

namespace Chaos::Private
{
	// Default traits for THashMappedArray that works for all ID/Element pairs where the 
	// ID has a MurmurFinalize32 implementation and we can compare equality of Elements and IDs.
	template<typename TIDType, typename TElementType>
	struct THashMappedArrayTraits
	{
		using FIDType = TIDType;
		using FElementType = TElementType;

		// Hash the ID to a 32 bit unsigned int for use with FHashTable
		static uint32 GetIDHash(const FIDType& ID)
		{
			return MurmurFinalize32(ID);
		}

		static FIDType GetElementID(const FElementType& Element)
		{
			return FIDType(Element);
		}
	};

	/**
	* A HashMap using FHashTable to index an array of elements of type TElementType, whcih should be uniquely identified by an object of type TIDType.
	* 
	* E.g.,
	*	using FMyDataID = int32;
	*	struct FMyData
	*	{
	*		FMyDataID ID;	// Every FMyData will require a unique ID if using the default THashMappedArrayTraits
	*		float MyValue;
	*	};
	* 
	*	const int32 HashTableSize = 128;				// Must be power of 2
	*	THashMapedArray<FMyDataID, FMyData> MyDataMap(HashTableSize);
	* 
	*	MyDataMap.Add(1, { 1, 1.0 });					// NOTE: ID passed twice. Once for the hash map and once to construct FMyData
	*	MyDataMap.Emplace(2, 2, 2.0);					// NOTE: ID passed twice. Once for the hash map and once for forwarding args to FMyData
	* 
	*	const FMyData* MyData2 = MyDataMap.Find(2);		// MyData2->MyValue == 2.0
	* 
	*/
	template<typename TIDType, typename TElementType, typename TTraits = THashMappedArrayTraits<TIDType, TElementType>>
	class THashMappedArray
	{
	public:
		using FIDType = TIDType;
		using FElementType = TElementType;
		using FTraits = TTraits;
		using FHashType = uint32;
		using FType = THashMappedArray<FIDType, FElementType, FTraits>;

		// Initialize the hash table. InHashSize must be a power of two (asserted)
		THashMappedArray(const int32 InHashSize)
			: HashTable(InHashSize)
		{
		}

		// Clear the hash map and reserve space for the specified number of elements (will not shrink)
		void Reset(const int32 InReserveElements)
		{
			check(InReserveElements >= 0);

			HashTable.Clear();

			// Resize will tightly fit the internal index allocation - only do this if the container grows
			if((uint32)InReserveElements > HashTable.GetIndexSize())
			{
				HashTable.Resize(InReserveElements);
			}

			Elements.Reset(InReserveElements);
		}

		// Try to add an element with the specified ID to the map. Does nothing if an Element with the same ID is already in the map.
		// Returns true if the element was added, false if the ID already existed.
		FORCEINLINE bool TryAdd(const FIDType ID, const FElementType& Element)
		{
			if (Find(ID) == nullptr)
			{
				Add(ID, Element);
				return true;
			}
			return false;
		}

		// Try to add an element with the specified ID to the map. Does nothing if an Element with the same ID is already in the map.
		// Returns true if the element was added, false if the ID already existed.
		template <typename... ArgsType>
		FORCEINLINE bool TryEmplace(const FIDType ID, ArgsType&&... Args)
		{
			if (Find(ID) == nullptr)
			{
				Emplace(ID, Forward<ArgsType>(Args)...);
				return true;
			}
			return false;
		}

		// Add an element with the specified ID to the map. Returns a reference to the newly inserted element.
		// It is an error to insert an item twice, but this is only detected in debug builds. Use TryAdd
		// if you do not know whether the items already exists in the map.
		FORCEINLINE FElementType& Add(const FIDType ID, const FElementType& Element)
		{
			checkSlow(Find(ID) == nullptr);

			const int32 Index = Elements.Add(Element);
			const FHashType Key = FTraits::GetIDHash(ID);

			HashTable.Add(Key, Index);

			return Elements[Index];
		}

		// Add an element with the specified ID to the map. Returns a reference to the newly inserted element.
		// It is an error to insert an item twice, but this is only detected in debug builds. Use TryEmplace
		// if you do not know whether the items already exists in the map.
		// NOTE: since your element type will also need to contain the ID, you usually have to pass the ID twice
		// to emplace, which is a little annoying, but shouldn't affect much.
		template <typename... ArgsType>
		FORCEINLINE FElementType& Emplace(const FIDType ID, ArgsType&&... Args)
		{
			checkSlow(Find(ID) == nullptr);

			const int32 Index = Elements.Emplace(Forward<ArgsType>(Args)...);
			const FHashType Key = FTraits::GetIDHash(ID);

			HashTable.Add(Key, Index);

			return Elements[Index];
		}

		void Remove(const FIDType ID)
		{
			const FHashType Key = FTraits::GetIDHash(ID);
			for (uint32 Index = HashTable.First(Key); HashTable.IsValid(Index); Index = HashTable.Next(Index))
			{
				if (FTraits::GetElementID(Elements[Index]) == ID)
				{
					const uint32 LastElemIndex = Elements.Num() - 1;

					// Remove the key/index from the hash table
					HashTable.Remove(Key, Index);

					// If this is the last element in the array (or if there's only one) then
					// no need for fancy swaps.
					if (Index == LastElemIndex)
					{
						Elements.RemoveAt(Index);
					}

					// If this is not the last element in the array, swap with the last element
					// and fix up the hash table
					else
					{
						const FIDType LastElemID = FTraits::GetElementID(Elements[LastElemIndex]);
						const FHashType LastElemKey = FTraits::GetIDHash(LastElemID);

						// Remove the last element's key/index from the hash table
						HashTable.Remove(LastElemKey, LastElemIndex);

						// Remove the element we want to remove, swapping it with the last element
						Elements.RemoveAtSwap(Index);

						// Re-add the last element's key, reusing the index of the element we just removed
						HashTable.Add(LastElemKey, Index);
					}

					break;
				}
			}
		}

		// Find the element with the specified ID. Roughly O(Max(1,N/M)) for N elements with a hash table of size M
		const FElementType* Find(const FIDType ID) const
		{
			return const_cast<FType*>(this)->Find(ID);
		}

		// Find the element with the specified ID. Roughly O(Max(1,N/M)) for N elements with a hash table of size M
		FElementType* Find(const FIDType ID)
		{
			const FHashType Key = FTraits::GetIDHash(ID);
			for (uint32 Index = HashTable.First(Key); HashTable.IsValid(Index); Index = HashTable.Next(Index))
			{
				if (FTraits::GetElementID(Elements[Index]) == ID)
				{
					return &Elements[Index];
				}
			}
			return nullptr;
		}

		// The number of elements that have been added to the map
		int32 Num() const
		{
			return Elements.Num();
		}

		// Get the element at ElementIndex (indexed by order in which they were added)
		FElementType& At(const int32 ElementIndex)
		{
			return Elements[ElementIndex];
		}

		// Get the element at ElementIndex (indexed by order in which they were added)
		const FElementType& At(const int32 ElementIndex) const
		{
			return Elements[ElementIndex];
		}

		// Move the array elements into an external array and reset
		TArray<FElementType> ExtractElements()
		{
			HashTable.Clear();
			return MoveTemp(Elements);
		}


	private:
		FHashTable HashTable;
		TArray<FElementType> Elements;
	};

}