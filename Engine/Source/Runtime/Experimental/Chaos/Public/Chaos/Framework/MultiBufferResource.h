// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

namespace Chaos
{
	enum class EMultiBufferMode : uint8
	{
		Undefined = 0,
		Single,
		Double,
		Triple,
		TripleGuarded
	};

	template<typename ResourceType>
	class IBufferResource
	{
	public:

		virtual ~IBufferResource() = default;

		virtual EMultiBufferMode GetBufferMode() = 0;
		virtual ResourceType* AccessProducerBuffer() = 0;

		//NOTE: these changes may not make it to producer side, it is meant for resource management not to pass information back
		virtual ResourceType* GetConsumerBufferMutable() = 0;

		virtual const ResourceType* GetProducerBuffer() const = 0;
		virtual const ResourceType* GetConsumerBuffer() const = 0;
		virtual void FlipProducer() = 0;
	};

	//////////////////////////////////////////////////////////////////////////
	/**
	 * Single Buffer Implementation
	 */
	template<typename ResourceType>
	class FSingleBuffer final : public IBufferResource<ResourceType>
	{
	public:

		FSingleBuffer() {}
		virtual ~FSingleBuffer() = default;

		virtual EMultiBufferMode GetBufferMode() override { return EMultiBufferMode::Single; }
		virtual ResourceType* AccessProducerBuffer() override { return &Data; }
		virtual ResourceType* GetConsumerBufferMutable() override {return &Data;}
		virtual const ResourceType* GetProducerBuffer() const override { return &Data; }
		virtual const ResourceType* GetConsumerBuffer() const override { return &Data; }

		void FlipProducer() override { /* NOP */ }

	private:

		ResourceType Data;
	};
	//////////////////////////////////////////////////////////////////////////
	/*
	* Double Buffer Implementation - Not thread-safe requires external locks
	*/
	template<typename ResourceType>
	class FDoubleBuffer final : public IBufferResource<ResourceType>
	{
	public:

		FDoubleBuffer()
		    : Data_Producer(&Data1)
		    , Data_Consumer(&Data2)
		{}
		virtual ~FDoubleBuffer() = default;

		virtual EMultiBufferMode GetBufferMode() override { return EMultiBufferMode::Double; }
		virtual ResourceType* AccessProducerBuffer() override { return Data_Producer; }
		virtual ResourceType* GetConsumerBufferMutable() override {return Data_Consumer;}
		virtual const ResourceType* GetProducerBuffer() const override { return Data_Producer; }
		virtual const ResourceType* GetConsumerBuffer() const override { return Data_Consumer; }

		void FlipProducer()
		{
			if (Data_Producer == &Data1)
			{
				Data_Producer = &Data2;
				Data_Consumer = &Data1;
			}
			else
			{
				Data_Producer = &Data1;
				Data_Consumer = &Data2;
			}
		}

	private:

		ResourceType Data1;
		ResourceType Data2;
		ResourceType* Data_Producer;
		ResourceType* Data_Consumer;
	};
	//////////////////////////////////////////////////////////////////////////
	/*
	* Triple Buffer Implementation - Not thread-safe requires external locks
	*/
	template<typename ResourceType>
	class FTripleBuffer final : public IBufferResource<ResourceType>
	{
	public:

		FTripleBuffer()
		    : WriteIndex(1)
		    , ReadIndex(0)
		{
		}
		virtual ~FTripleBuffer() = default;

		virtual EMultiBufferMode GetBufferMode() override { return EMultiBufferMode::Triple; }
		virtual ResourceType* AccessProducerBuffer() override { return &Data[GetWriteIndex()]; }
		virtual ResourceType* GetConsumerBufferMutable() override {return &Data[GetReadIndex()];}

		virtual const ResourceType* GetProducerBuffer() const override { return &Data[GetWriteIndex()]; }
		virtual const ResourceType* GetConsumerBuffer() const override { return &Data[GetReadIndex()]; }

		virtual void FlipProducer() override
		{
			int32 CurrentReadIdx = GetReadIndex();
			int32 CurrentWriteIdx = GetWriteIndex();
			int32 FreeIdx = 3 - (CurrentReadIdx + CurrentWriteIdx);

			ReadIndex.Store(CurrentWriteIdx);
			WriteIndex.Store(FreeIdx);

			checkSlow(GetReadIndex() != GetWriteIndex());
		}

	private:
		int32 GetWriteIndex() const { return WriteIndex.Load(); }
		int32 GetReadIndex() const { return ReadIndex.Load(); }

		ResourceType Data[3];
		TAtomic<int32> WriteIndex;
		TAtomic<int32> ReadIndex;
	};
	//////////////////////////////////////////////////////////////////////////
	/**
	 * Triple buffer based on a single atomic variable, that guards against 
	 * the consumer thread using old values.
	 *
	 * Not thread-safe, requires external locks.
	 */
	template<typename ResourceType>
	class FGuardedTripleBuffer final : public IBufferResource<ResourceType>
	{
	public:
		/** This class implements a circular buffer access pattern, such that during
		 * normal serial operation each buffer will be used.  However, we can alter
		 * how we use the API to use only 2 or 1 buffer:
		 * 
		 *    SERIAL ACCESS BUFFER USE - NORMAL OPERATION:
		 *            P  C  I
		 *     Init:  0, 1, 2
		 *     
		 *     AccP: *0, 1, 2 - *write access
		 *     Flip:  2, 1, 0
		 *     Cons: 2, ^0, 1 - ^read access
		 *     
		 *     AccP: *2, 0, 1
		 *     Flip:  1, 0, 2
		 *     Cons: 1, ^2, 0
		 *     
		 *     AccP: *1, 2, 0
		 *     Flip:  0, 2, 1
		 *     Cons: 0, ^1, 2
		 * 
		 * 
		 *    SERIAL ACCESS BUFFER USE - DOUBLE BUFFER EMULATION:
		 *     Init:  0, 1, 2
		 *     
		 *     AccP: *0, 1, 2 - *write access
		 *     Flip:  2, 1, 0
		 *     Cons: 2, ^0, 1 - ^read access
		 *     Flip:  1, 0, 2		<- Add an extra flip and only use 2 buffers.
		 *     
		 *     AccP: *1, 0, 2
		 *     Flip:  2, 0, 1
		 *     Cons: 2, ^1, 0
		 *     Flip:  0, 1, 2
		 * 
		 *
		 *    SERIAL ACCESS BUFFER USE - SINGLE BUFFER EMULATION:
		 *     Init:  0, 1, 2
		 *
		 *     AccP: *0, 1, 2 - *write access
		 *     Flip:  2, 1, 0
		 *     Cons: 2, ^0, 1 - ^read access
		 *     Cons:  2, 1, 0		<- Add an extra consume...
		 *     Flip:  0, 1, 2		<- and an extra flip and only use 1 buffer.
		 */

		FGuardedTripleBuffer()
		    : ProducerThreadBuffer(&Buffers[0])
		    , ConsumerThreadBuffer(&Buffers[1])
		    , Interchange(&Buffers[2])
		{}

		virtual EMultiBufferMode GetBufferMode() override { return EMultiBufferMode::Triple; }

		/**
		 * Get the current producer buffer for writing.
		 *
		 * The returned pointer will never be null.
		 *
		 * This function should only be called by the producer thread.
		 */
		virtual ResourceType* AccessProducerBuffer() override
		{
			return ProducerThreadBuffer->Value.Get();
		}

		virtual ResourceType* GetConsumerBufferMutable() override
		{
			check(false);
			return nullptr;
		}

		/**
		 * Get the current producer buffer.
		 *
		 * The returned pointer will never be null.
		 *
		 * This function should only be called by the producer thread.
		 */
		virtual const ResourceType* GetProducerBuffer() const override
		{
			return ProducerThreadBuffer->Value.Get();
		}

		/**
		 * Get an updated buffer for the consuming thread to read from.
		 *
		 * The returned pointer may be null if the producer thread hasn't done 
		 * an update since the last time this function was called.
		 *
		 * This function should only be called by the consumer thread.
		 */
		virtual const ResourceType* GetConsumerBuffer() const override
		{
			ConsumerThreadBuffer = Interchange.Exchange(ConsumerThreadBuffer);
			// Return null if the value hasn't changed.
			if (!ConsumerThreadBuffer->bValid)
			{
				return nullptr;
			}
			// Mark the buffer as invalid so we don't use this value again.
			ConsumerThreadBuffer->bValid = false;
			return ConsumerThreadBuffer->Value.Get();
		}

		/**
		 * Get access to the currently held consumer buffer, ignoring whether
		 * it's already been consumed.
		 */
		const ResourceType* PeekConsumerBuffer() const
		{
			return ConsumerThreadBuffer->Value.Get();
		}

		/**
		 * Make the current producer buffer available to the consumer thread.
		 *
		 * This function should only be called by the producer thread.
		 */
		virtual void FlipProducer()
		{
			ProducerThreadBuffer->bValid = true; // In with the new...
			ProducerThreadBuffer = Interchange.Exchange(ProducerThreadBuffer);
			ProducerThreadBuffer->bValid = false; // Out with the old.
		}

	private:
		struct ResourceTypeWrapper
		{
			ResourceTypeWrapper()
			    : Value(new ResourceType())
			    , bValid(false)
			{}
			ResourceTypeWrapper(const ResourceTypeWrapper&) = delete;
			ResourceTypeWrapper(ResourceTypeWrapper&& Other)
			    : Value(MoveTemp(Other.Value))
			    , bValid(MoveTemp(Other.bValid))
			{}

			void operator=(const ResourceTypeWrapper& Other) = delete;

			TUniquePtr<ResourceType> Value;
			bool bValid;
		};

		ResourceTypeWrapper Buffers[3] = {ResourceTypeWrapper(), ResourceTypeWrapper(), ResourceTypeWrapper()};
		ResourceTypeWrapper* ProducerThreadBuffer;
		mutable ResourceTypeWrapper* ConsumerThreadBuffer;
		mutable TAtomic<ResourceTypeWrapper*> Interchange;
	};

	//////////////////////////////////////////////////////////////////////////

	template<typename ResourceType>
	class FMultiBufferFactory
	{
	public:
		static TUniquePtr<IBufferResource<ResourceType>> CreateBuffer(const EMultiBufferMode BufferMode)
		{
			switch (BufferMode)
			{
			case EMultiBufferMode::Single:
			{
				return MakeUnique<FSingleBuffer<ResourceType>>();
			}
			break;

			case EMultiBufferMode::Double:
			{
				return MakeUnique<FDoubleBuffer<ResourceType>>();
			}
			break;

			case EMultiBufferMode::Triple:
			{
				return MakeUnique<FTripleBuffer<ResourceType>>();
			}
			break;

			case EMultiBufferMode::TripleGuarded:
			{
				return MakeUnique<FGuardedTripleBuffer<ResourceType>>();
			}
			break;

			default:
				checkf(false, TEXT("FMultiBufferFactory Unexpected buffer mode"));
				break;

			}

			return nullptr;
		}
	};

} // namespace Chaos
