// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/Function.h"

namespace Chaos
{
	void CHAOS_API PhysicsParallelForRange(int32 InNum, TFunctionRef<void(int32, int32)> InCallable, const int32 MinBatchSize, bool bForceSingleThreaded = false);
	void CHAOS_API PhysicsParallelFor(int32 InNum, TFunctionRef<void(int32)> InCallable, bool bForceSingleThreaded = false);
	void CHAOS_API InnerPhysicsParallelForRange(int32 InNum, TFunctionRef<void(int32, int32)> InCallable, const int32 MinBatchSize, bool bForceSingleThreaded = false);
	void CHAOS_API InnerPhysicsParallelFor(int32 InNum, TFunctionRef<void(int32)> InCallable, bool bForceSingleThreaded = false);
	void CHAOS_API PhysicsParallelForWithContext(int32 InNum, TFunctionRef<int32 (int32, int32)> InContextCreator, TFunctionRef<void(int32, int32)> InCallable, bool bForceSingleThreaded = false);
	//void CHAOS_API PhysicsParallelFor_RecursiveDivide(int32 InNum, TFunctionRef<void(int32)> InCallable, bool bForceSingleThreaded = false);

	/**
	 * Helper to call from task execution logic.
	 * Returns true if the tasks should be run, or if it is better to stay on a single thread and not spawn tasks.
	 * @param InInstanceNum The total number of instance that should be spread through the tasks.  It will be compared to GlobalSmallBatchSize.
	 */
	bool CHAOS_API ShouldExecuteTasks(int32 InInstanceNum);

	CHAOS_API extern bool bSingleWorkerPhysics;
	CHAOS_API extern int32 MaxNumWorkers;
	CHAOS_API extern int32 CollisionSmallBatchSize;
	CHAOS_API extern int32 GlobalSmallBatchSize;
	CHAOS_API extern int32 LargeBatchSize;
	CHAOS_API extern bool bDisablePhysicsParallelFor;
	CHAOS_API extern bool bDisableParticleParallelFor;
	CHAOS_API extern bool bDisableCollisionParallelFor;
}