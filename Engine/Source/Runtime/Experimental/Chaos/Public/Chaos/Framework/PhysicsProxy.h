// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/ArrayCollectionArray.h"
#include "Chaos/Declares.h"
#include "Chaos/Framework/PhysicsSolverBase.h"
#include "Chaos/Framework/PhysicsProxyBase.h"
#include "UObject/GCObject.h"
//
// NOTE: This file is widely included in Engine code. 
// Avoid including Chaos headers when possible.
//

struct FKinematicProxy;
class FFieldSystemCommand;
struct FBodyInstance;

namespace Chaos
{
	template<typename T, int D> class TPBDRigidParticles;

	using FPBDRigidParticles = TPBDRigidParticles<FReal, 3>;
}

/**
 * Base object interface for solver objects. Defines the expected API for objects
 * uses CRTP for static dispatch, entire API considered "pure-virtual" and must be* defined.
 * Forgetting to implement any of the interface functions will give errors regarding
 * recursion on all control paths for TPhysicsProxy<T> where T will be the type
 * that has not correctly implemented the API.
 *
 * PersistentTask uses IPhysicsProxyBase, so when implementing a new specialized type
 * it is necessary to include its header file in PersistentTask.cpp allowing the linker
 * to properly resolve the new type. 
 *
 * May not be necessary overall once the engine has solidified - we can just use the
 * final concrete objects but this gives us almost the same flexibility as the old
 * callbacks while solving most of the drawbacks (virtual dispatch, cross-object interaction)
 *
 * #BG TODO - rename the callbacks functions, document for the base solver object
 */
template<class Concrete, class ConcreteData, typename TProxyTimeStamp>
class TPhysicsProxy : public IPhysicsProxyBase
{

public:
	using FParticleType = Concrete;

	using FParticlesType = Chaos::FPBDRigidParticles;
	using FIntArray = Chaos::TArrayCollectionArray<int32>;

	TPhysicsProxy()
		: IPhysicsProxyBase(ConcreteType(), nullptr, MakeShared<TProxyTimeStamp>())
	{
	}

	explicit TPhysicsProxy(UObject* InOwner)
		: IPhysicsProxyBase(ConcreteType(), InOwner, MakeShared<TProxyTimeStamp>())
	{
	}

	/** Virtual destructor for derived objects, ideally no other virtuals should exist in this chain */
	virtual ~TPhysicsProxy() = default;

	/**
	 * The following functions are to be implemented by all solver objects as we're using CRTP / F-Bound to
	 * statically dispatch the calls. Any common functions should be added here and to the derived solver objects
	 */

	// Previously callback related functions, all called in the context of the physics thread if enabled.
	bool IsSimulating() const { return static_cast<const Concrete*>(this)->IsSimulating(); }
	void UpdateKinematicBodiesCallback(const FParticlesType& InParticles, const float InDt, const float InTime, FKinematicProxy& InKinematicProxy) { static_cast<Concrete*>(this)->UpdateKinematicBodiesCallback(InParticles, InDt, InTime, InKinematicProxy); }
	void StartFrameCallback(const float InDt, const float InTime) { static_cast<Concrete*>(this)->StartFrameCallback(InDt, InTime); }
	void EndFrameCallback(const float InDt) { static_cast<Concrete*>(this)->EndFrameCallback(InDt); }
	void CreateRigidBodyCallback(FParticlesType& InOutParticles) { static_cast<Concrete*>(this)->CreateRigidBodyCallback(InOutParticles); }
	void DisableCollisionsCallback(TSet<TTuple<int32, int32>>& InPairs) { static_cast<Concrete*>(this)->DisableCollisionsCallback(InPairs); }
	void AddForceCallback(FParticlesType& InParticles, const float InDt, const int32 InIndex) { static_cast<Concrete*>(this)->AddForceCallback(InParticles, InDt, InIndex); }

	/** The Particle Binding creates a connection between the particles in the simulation and the solver objects dataset. */
	void BindParticleCallbackMapping(Chaos::TArrayCollectionArray<PhysicsProxyWrapper> & PhysicsProxyReverseMap, Chaos::TArrayCollectionArray<int32> & ParticleIDReverseMap) {static_cast<Concrete*>(this)->BindParticleCallbackMapping(PhysicsProxyReverseMap, ParticleIDReverseMap);}

	/** Returns the concrete type of the derived class*/
	static constexpr EPhysicsProxyType ConcreteType() { return Concrete::ConcreteType(); }
	
	/**
	 * CONTEXT: GAMETHREAD
	* Returns a new unmanaged allocation of the data saved on the handle, otherwise nullptr
	*/
	//ConcreteData* NewData() { return static_cast<Concrete*>(this)->NewData(); }

	/**
	* CONTEXT: GAMETHREAD -> to -> PHYSICSTHREAD
	* Called on the game thread when the solver is about to advance forward. This
	* callback should Enqueue commands on the PhysicsThread to update the state of
	* the solver
	*/
	//void PushToPhysicsState(const ConcreteData* InData) { static_cast<Concrete*>(this)->PushToPhysicsState(InData); }

	/**
	* CONTEXT: GAMETHREAD
	* Called on game thread after NewData has been called to buffer the particle data
	* for physics. The purpose of this method is to clear data, such as external force
	* and torque, which have been accumulated over a game tick. Buffering these values
	* once means they'll be accounted for in physics. If they are not cleared, then
	* they may "overaccumulate".
	*/
	void ClearAccumulatedData() { static_cast<Concrete*>(this)->ClearAccumulatedData(); }
	
	/**
	 * CONTEXT: GAMETHREAD
	 * Called during the gamethread sync after the proxy has been removed from its solver
	 * intended for final handoff of any data the proxy has that the gamethread may
	 * be interested in
	*/
	void SyncBeforeDestroy() { static_cast<Concrete*>(this)->SyncBeforeDestroy(); }

	/**
	 * CONTEXT: PHYSICSTHREAD
	 * Called on the physics thread when the engine is shutting down the proxy and we need to remove it from
	 * any active simulations. Proxies are expected to entirely clean up their simulation
	 * state within this method. This is run in the task command step by the scene
	 * so the simulation will currently be idle
	 */
	void OnRemoveFromScene() { static_cast<Concrete*>(this)->OnRemoveFromScene(); }

	bool IsDirty() { return static_cast<Concrete*>(this)->IsDirty(); }

	void* GetUserData() const { return nullptr; }

	Chaos::FRigidTransform3 GetTransform() const { return Chaos::FRigidTransform3(); }
	
	FORCEINLINE_DEBUGGABLE TProxyTimeStamp& GetSyncTimestampTyped()
	{
		return GetSyncTimestampAs<TProxyTimeStamp>();
	}
};
