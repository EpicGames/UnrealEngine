// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/Declares.h"
#include "UObject/GCObject.h"
#include "Chaos/Core.h"
#include "HAL/IConsoleManager.h"

enum class EPhysicsProxyType : uint32
{
	NoneType = 0,
	StaticMeshType = 1,
	GeometryCollectionType = 2,
	FieldType = 3,
	SkeletalMeshType = 4,
	JointConstraintType = 8,	//left gap when removed some types in case these numbers actually matter to someone, should remove
	SuspensionConstraintType = 9,
	CharacterGroundConstraintType = 10,
	SingleParticleProxy,
	ClusterUnionProxy,
	Count
};

namespace Chaos
{
	class FPhysicsSolverBase;
}

namespace RenderInterpolationCVars
{
	extern float RenderInterpErrorCorrectionDuration;
	extern float RenderInterpMaximumErrorCorrectionBeforeSnapping;
	extern float RenderInterpMaximumErrorCorrectionDesyncTimeBeforeSnapping;
	extern float RenderInterpErrorVelocitySmoothingDuration;
	extern float RenderInterpErrorDirectionalDecayMultiplier;
	extern bool bRenderInterpErrorVelocityCorrection;
	extern bool bRenderInterpDebugDraw;
	extern bool bRenderInterpDebugDrawGC;
	extern float RenderInterpDebugDrawZOffset;
	extern bool bRenderInterpApplyExponentialDecay;
	extern float RenderInterpExponentialDecayLinearHalfLife;
	extern float RenderInterpExponentialDecayAngularHalfLife;
	extern float RenderInterpMinimumLinearThreshold;
	extern float RenderInterpMinimumAngularThreshold;
	extern bool bSnapToLatest;
}

enum class EProxyInterpolationType : uint32
{
	Base = 0,
	ErrorLinear = 1,
	ErrorVelocity = 2
};

struct FProxyTimestampBase
{
	bool bDeleted = false;
};

template <typename TPropertyType>
struct TTimestampProperty
{
	FORCEINLINE_DEBUGGABLE void Set(int32 InTimestamp, const TPropertyType& InValue)
	{
		Value = InValue;
		Timestamp = InTimestamp;
	}
	
	TPropertyType Value;
	int32 Timestamp = INDEX_NONE;
};

struct FSingleParticleProxyTimestamp: public FProxyTimestampBase
{
	int32 ObjectStateTimestamp = INDEX_NONE;
	TTimestampProperty<Chaos::FVec3> OverWriteX;
	TTimestampProperty<Chaos::FRotation3> OverWriteR;
	TTimestampProperty<Chaos::FVec3> OverWriteV;
	TTimestampProperty<Chaos::FVec3> OverWriteW;
};

struct FGeometryCollectionProxyTimestamp: public FProxyTimestampBase
{
	// nothing to add as Geometry collections are driven from the Physics thread only
	// ( including kinematic targeting )
};

struct FClusterUnionProxyTimestamp : public FProxyTimestampBase
{
	TTimestampProperty<Chaos::FVec3> OverWriteX;
	TTimestampProperty<Chaos::FRotation3> OverWriteR;
	TTimestampProperty<Chaos::FVec3> OverWriteV;
	TTimestampProperty<Chaos::FVec3> OverWriteW;
};

class IPhysicsProxyBase
{
public:
	IPhysicsProxyBase(EPhysicsProxyType InType, UObject* InOwner, TSharedPtr<FProxyTimestampBase,ESPMode::ThreadSafe> InProxyTimeStamp)
		: Solver(nullptr)
		, Owner(InOwner)
		, DirtyIdx(INDEX_NONE)
		, Type(InType)
		, SyncTimestamp(InProxyTimeStamp)
	{}

	UObject* GetOwner() const { return Owner; }

	template< class SOLVER_TYPE>
	SOLVER_TYPE* GetSolver() const { return static_cast<SOLVER_TYPE*>(Solver); }

	Chaos::FPhysicsSolverBase* GetSolverBase() const { return Solver; }

	//Should this be in the public API? probably not
	template< class SOLVER_TYPE = Chaos::FPhysicsSolver>
	void SetSolver(SOLVER_TYPE* InSolver) { Solver = InSolver; }

	EPhysicsProxyType GetType() const { return Type; }

	//todo: remove this
	virtual void* GetHandleUnsafe() const { check(false); return nullptr; }

	int32 GetDirtyIdx() const { return DirtyIdx; }
	void SetDirtyIdx(const int32 Idx) { DirtyIdx = Idx; }
	void ResetDirtyIdx() { DirtyIdx = INDEX_NONE; }

	void MarkDeleted() { SyncTimestamp->bDeleted = true; }
	bool GetMarkedDeleted() const { return SyncTimestamp->bDeleted; }

	TSharedPtr<FProxyTimestampBase,ESPMode::ThreadSafe> GetSyncTimestamp() const { return SyncTimestamp; }

	bool IsInitialized() const { return InitializedOnStep != INDEX_NONE; }
	void SetInitialized(const int32 InitializeStep)
	{
		//If changed initialization, ignore the very first initialization push data
		if(InitializedOnStep != InitializeStep && InitializedOnStep != INDEX_NONE)
		{
			IgnoreDataOnStep_Internal = InitializedOnStep;
		}

		InitializedOnStep = InitializeStep;
	}
	int32 GetInitializedStep() const { return InitializedOnStep; }

	int32 GetIgnoreDataOnStep_Internal() const { return IgnoreDataOnStep_Internal; }

	IPhysicsProxyBase* GetParentProxy() const { return ParentProxy; }
	void SetParentProxy(IPhysicsProxyBase* InProxy) { ParentProxy = InProxy; }

	// Render Interpolation CVars
	UE_DEPRECATED(5.5, "Deprecated, use RenderInterpolationCVars::RenderInterpErrorCorrectionDuration")
	static float GetRenderInterpErrorCorrectionDuration()
	{
		return RenderInterpolationCVars::RenderInterpErrorCorrectionDuration;
	}
	UE_DEPRECATED(5.5, "Deprecated, use RenderInterpolationCVars::RenderInterpMaximumErrorCorrectionBeforeSnapping")
	static float GetRenderInterpMaximumErrorCorrectionBeforeSnapping()
	{
		return RenderInterpolationCVars::RenderInterpMaximumErrorCorrectionBeforeSnapping;
	}
	UE_DEPRECATED(5.5, "Deprecated, use RenderInterpolationCVars::RenderInterpErrorVelocitySmoothingDuration")
	static float GetRenderInterpErrorVelocitySmoothingDuration()
	{
		return RenderInterpolationCVars::RenderInterpErrorVelocitySmoothingDuration;
	}
	UE_DEPRECATED(5.5, "Deprecated, use RenderInterpolationCVars::bRenderInterpDebugDraw")
	static bool GetRenderInterpDebugDraw()
	{
		return RenderInterpolationCVars::bRenderInterpDebugDraw;
	}
	UE_DEPRECATED(5.5, "Deprecated, use RenderInterpolationCVars::RenderInterpErrorDirectionalDecayMultiplier")
	static float GetRenderInterpErrorDirectionalDecayMultiplier()
	{
		return RenderInterpolationCVars::RenderInterpErrorDirectionalDecayMultiplier;
	}

protected:
	// Ensures that derived classes can successfully call this destructor
	// but no one can delete using a IPhysicsProxyBase*
	CHAOS_API virtual ~IPhysicsProxyBase();

	template<typename TProxyTimeStamp>
	FORCEINLINE_DEBUGGABLE TProxyTimeStamp& GetSyncTimestampAs()
	{
		return static_cast<TProxyTimeStamp&>(*GetSyncTimestamp());
	}
	
	/** The solver that owns the solver object */
	Chaos::FPhysicsSolverBase* Solver;
	UObject* Owner;

private:
	int32 DirtyIdx;
protected:
	/** Proxy type */
	EPhysicsProxyType Type;
private:
	TSharedPtr<FProxyTimestampBase,ESPMode::ThreadSafe> SyncTimestamp;
	IPhysicsProxyBase* ParentProxy = nullptr;
protected:
	int32 InitializedOnStep = INDEX_NONE;
	int32 IgnoreDataOnStep_Internal = INDEX_NONE;

	CHAOS_API int32 GetSolverSyncTimestamp_External() const;
};

struct PhysicsProxyWrapper
{
	IPhysicsProxyBase* PhysicsProxy;
	EPhysicsProxyType Type;
};

struct FErrorInterpolationSettings
{
	FErrorInterpolationSettings()
		: ErrorCorrectionDuration(0.3f)
		, MaximumErrorCorrectionBeforeSnapping(250.0f)
		, MaximumErrorCorrectionDesyncTimeBeforeSnapping(0.6f)
		, ErrorDirectionalDecayMultiplier(0.0f)
		, bApplyExponentialDecay(false)
		, ExponentialDecayLinearHalfLife(0.06f)
		, ExponentialDecayAngularHalfLife(0.06f)
		, MinimumLinearThreshold(0.1f)
		, MinimumAngularThreshold(0.001f)
	{}
	~FErrorInterpolationSettings() = default;

	/** How long in seconds to apply error correction over */
	float ErrorCorrectionDuration;

	/** Maximum error correction distance before we stop interpolating and snap to target. */
	float MaximumErrorCorrectionBeforeSnapping;

	/** Time multiplied by the particles velocity to get the distance that error correction will be performed within without snapping, disable by setting a negative value
	* NOTE: At lower velocities MaximumErrorCorrectionBeforeSnapping will act as a lowest distance clamp. */
	float MaximumErrorCorrectionDesyncTimeBeforeSnapping;

	/** Decay error offset in the direction that the physics object is moving, value is multiplier of projected offset direction, 0.25 means a 25 % decay of the magnitude in the direction of physics travel.Deactivate by setting to 0 */
	float ErrorDirectionalDecayMultiplier;

	/** When enabled a post-resim error will decay exponentially (instead of linearly) based on half-life time set in ExponentialDecayLinearHalfLife and ExponentialDecayAngularHalfLife. */
	bool bApplyExponentialDecay;

	/** Sets the positional half-life time for when bApplyExponentialDecay is enabled. */
	float ExponentialDecayLinearHalfLife;

	/** Sets the rotational half-life time for when bApplyExponentialDecay is enabled. */
	float ExponentialDecayAngularHalfLife;

	/** Squared value, when the remaining render error is below this we clear it, if ApplyExponentialDecay is enabled. */
	float MinimumLinearThreshold;

	/** When the remaining render error angle is below this we clear it, if ApplyExponentialDecay is enabled. */
	float MinimumAngularThreshold;
};

struct FProxyInterpolationBase
{
	FProxyInterpolationBase(const int32 PullDataInterpIdx = INDEX_NONE, const int32 InterpChannel = 0)
		: PullDataInterpIdx_External(PullDataInterpIdx)
		, InterpChannel_External(InterpChannel)
	{}
	virtual ~FProxyInterpolationBase() = default;

	int32 GetPullDataInterpIdx_External() const { return PullDataInterpIdx_External; }
	void SetPullDataInterpIdx_External(const int32 Idx) { PullDataInterpIdx_External = Idx; }

	int32 GetInterpChannel_External() const { return InterpChannel_External; }
	void SetInterpChannel_External(const int32 Channel) { InterpChannel_External = Channel; }

protected:
	int32 PullDataInterpIdx_External;
	int32 InterpChannel_External;

public:
	// --- Error correction interpolation API ---

	/** This interpolation structs type */
	static const EProxyInterpolationType InterpolationType = EProxyInterpolationType::Base;

	/** Get this interpolation structs type */
	virtual const EProxyInterpolationType GetInterpolationType() const { return FProxyInterpolationBase::InterpolationType; }

	/** If currently correcting an error through interpolation */
	virtual const bool IsErrorSmoothing() const { return false; }

	/** Get the position of the current error correction, taking current Alpha between GT and PT into account */
	virtual const Chaos::FVec3 GetErrorX(const Chaos::FRealSingle Alpha) const { return Chaos::FVec3::ZeroVector; }

	/** Get the rotation of the current error correction, taking current Alpha between GT and PT into account */
	virtual const FQuat GetErrorR(const Chaos::FRealSingle Alpha) const { return FQuat::Identity; }

	/** Add X and R error onto current error to correct through interpolation */
	virtual void AccumulateErrorXR(const Chaos::FVec3 X, const FQuat R, const int32 CurrentSimTick, const int32 ErrorSmoothDuration) { }

	/** Tick current error data and decay error */
	virtual const bool UpdateError(const int32 CurrentSimTick, const Chaos::FReal AsyncFixedTimeStep) { return false; }

	/** EXPERIMENTAL - Decay error based on moved direction and distance */
	UE_DEPRECATED(5.6, "Deprecated, use DirectionalDecay(Direction, ErrorDirectionalDecayMultiplier) instead")
	virtual const bool DirectionalDecay(Chaos::FVec3 Direction) { return DirectionalDecay(Direction, 0.0f); }

	/** EXPERIMENTAL - Decay error based on moved direction and distance
	* @param ErrorDirectionalDecayMultiplier is a multiplier where 0.25 means a 25% decay along the direction of physics movement that aligns with the error offset direction, parameter gets overridden if there are custom settings for this particle */
	virtual const bool DirectionalDecay(Chaos::FVec3 Direction, const float ErrorDirectionalDecayMultiplier) { return false; }

	/** EXPERIMENTAL - If currently correcting error while taking velocity into account */
	virtual const bool IsErrorVelocitySmoothing() const { return false; }

	/** EXPERIMENTAL - Returns the Alpha of how much to take previous velocity into account, used to lerp from linear extrapolation to the predicted position based on previous velocity */
	virtual const Chaos::FRealSingle GetErrorVelocitySmoothingAlpha(const int32 ErrorVelocitySmoothDuration) const { return 0; }

	/** EXPERIMENTAL - Get the position of the velocity-based correction, taking current Alpha between GT and PT into account*/
	virtual const Chaos::FVec3 GetErrorVelocitySmoothingX(const Chaos::FRealSingle Alpha) const { return Chaos::FVec3::ZeroVector; }

	/** EXPERIMENTAL - Register the current velocity and position for use in velocity correction calculations */
	virtual void SetVelocitySmoothing(const Chaos::FVec3 CurrV, const Chaos::FVec3 CurrX, const int32 ErrorVelocitySmoothDuration) { }

	/** Get FErrorInterpolationSettings which stores custom settings for render interpolation error corrections, returns nullptr if there are no custom settings */
	virtual FErrorInterpolationSettings* GetErrorInterpolationSettings() { return nullptr; }

	/** Get FErrorInterpolationSettings which stores custom settings for render interpolation error corrections, returns nullptr if there are no custom settings */
	virtual const FErrorInterpolationSettings* GetErrorInterpolationSettings() const { return nullptr; }

	/** Reset FErrorInterpolationSettings */
	virtual void ResetErrorInterpolationSettings() { }
};

// Render interpolation that can correct errors from resimulation / repositions through a linear decay over N simulation tick.
struct FProxyInterpolationError : FProxyInterpolationBase
{
	using Super = FProxyInterpolationBase;

	FProxyInterpolationError(const int32 PullDataInterpIdx = INDEX_NONE, const int32 InterpChannel = 0) : Super(PullDataInterpIdx, InterpChannel)
	{}
	virtual ~FProxyInterpolationError() = default;

	static const EProxyInterpolationType InterpolationType = EProxyInterpolationType::ErrorLinear;
	virtual const EProxyInterpolationType GetInterpolationType() const override { return FProxyInterpolationError::InterpolationType; }

	CHAOS_API virtual const bool IsErrorSmoothing() const override;

	virtual const Chaos::FVec3 GetErrorX(const Chaos::FRealSingle Alpha) const override { return FMath::Lerp(ErrorXPrev, ErrorX, Alpha); }

	virtual const FQuat GetErrorR(const Chaos::FRealSingle Alpha) const override { return FQuat::Slerp(ErrorRPrev, ErrorR, Alpha); }

	CHAOS_API virtual void AccumulateErrorXR(const Chaos::FVec3 X, const FQuat R, const int32 CurrentSimTick, const int32 ErrorSmoothDuration) override;

	CHAOS_API virtual const bool UpdateError(const int32 CurrentSimTick, const Chaos::FReal AsyncFixedTimeStep) override;

	CHAOS_API virtual const bool DirectionalDecay(Chaos::FVec3 Direction, float ErrorDirectionalDecayMultiplier) override;
	
protected:
	CHAOS_API virtual const bool DecayError(const Chaos::FReal AsyncFixedTimeStep);

	virtual void Reset()
	{
		ErrorX = Chaos::FVec3::ZeroVector;
		ErrorXPrev = Chaos::FVec3::ZeroVector;
		ErrorR = FQuat::Identity;
		ErrorRPrev = FQuat::Identity;
		EndDecayTick = 0;
		LastSimTick = 0;
		SimTicks = 0;
	}

public:
	/** Get or create FErrorInterpolationSettings to stores custom settings for render interpolation error corrections */
	FErrorInterpolationSettings& GetOrCreateErrorInterpolationSettings()
	{
		if (!ErrorInterpolationSettings.IsValid())
		{
			ErrorInterpolationSettings = MakeUnique<FErrorInterpolationSettings>();
		}
		return *ErrorInterpolationSettings.Get();
	}

	/** Get FErrorInterpolationSettings which stores custom settings for render interpolation error corrections, returns nullptr if there are no custom settings */
	virtual FErrorInterpolationSettings* GetErrorInterpolationSettings() override
	{
		return ErrorInterpolationSettings.Get();
	}
	
	virtual const FErrorInterpolationSettings* GetErrorInterpolationSettings() const override
    {
		return ErrorInterpolationSettings.Get();
	}

	virtual void ResetErrorInterpolationSettings() override
	{
		ErrorInterpolationSettings.Reset();
	}	

protected:
	int32 LastSimTick = 0;
	int32 SimTicks = 0;
	int32 EndDecayTick = 0;

	Chaos::FVec3 ErrorX = Chaos::FVec3::Zero();
	Chaos::FVec3 ErrorXPrev = Chaos::FVec3::Zero();
	FQuat ErrorR = FQuat::Identity;
	FQuat ErrorRPrev = FQuat::Identity;

	TUniquePtr<FErrorInterpolationSettings> ErrorInterpolationSettings = nullptr;
};


// Render Interpolation that both perform the linear error correction from FProxyInterpolationError and takes incoming velocity into account to make a smoother and more organic correction of the error.
struct FProxyInterpolationErrorVelocity : FProxyInterpolationError
{
	using Super = FProxyInterpolationError;

	FProxyInterpolationErrorVelocity(const int32 PullDataInterpIdx = INDEX_NONE, const int32 InterpChannel = 0) : Super(PullDataInterpIdx, InterpChannel)
	{}
	virtual ~FProxyInterpolationErrorVelocity() = default;

	static const EProxyInterpolationType InterpolationType = EProxyInterpolationType::ErrorVelocity;
	virtual const EProxyInterpolationType GetInterpolationType() const override { return FProxyInterpolationErrorVelocity::InterpolationType; }

	virtual const bool IsErrorVelocitySmoothing() const override { return ErrorVelocitySmoothingCount > 0; }
	virtual const Chaos::FRealSingle GetErrorVelocitySmoothingAlpha(const int32 ErrorVelocitySmoothDuration) const override { return Chaos::FRealSingle(ErrorVelocitySmoothingCount) / Chaos::FRealSingle(ErrorVelocitySmoothDuration); }
	virtual const Chaos::FVec3 GetErrorVelocitySmoothingX(const Chaos::FRealSingle Alpha) const override { return FMath::Lerp(ErrorVelocitySmoothingXPrev, ErrorVelocitySmoothingX, Alpha); }

	virtual const bool UpdateError(const int32 CurrentSimTick, const Chaos::FReal AsyncFixedTimeStep) override
	{		
		if (Super::UpdateError(CurrentSimTick, AsyncFixedTimeStep))
		{
			StepErrorVelocitySmoothingData(AsyncFixedTimeStep);
			return true;
		}
		return false;
	}

	virtual void SetVelocitySmoothing(const Chaos::FVec3 CurrV, const Chaos::FVec3 CurrX, const int32 ErrorVelocitySmoothDuration) override
	{
		// Cache pre error velocity and position to be used when smoothing out error correction
		ErrorVelocitySmoothingV = CurrV;
		ErrorVelocitySmoothingX = CurrX;
		ErrorVelocitySmoothingXPrev = ErrorVelocitySmoothingX;
		ErrorVelocitySmoothingCount = ErrorVelocitySmoothDuration;
	}

protected:
	virtual void StepErrorVelocitySmoothingData(const Chaos::FReal AsyncFixedTimeStep)
	{
		// Step the error velocity smoothing position forward along the previous velocity to have a new position to base smoothing on each tick
		if (IsErrorVelocitySmoothing())
		{
			const Chaos::FReal Time = AsyncFixedTimeStep * SimTicks;
			ErrorVelocitySmoothingXPrev = ErrorVelocitySmoothingX;
			ErrorVelocitySmoothingX += ErrorVelocitySmoothingV * Time;

			ErrorVelocitySmoothingCount = FMath::Max(ErrorVelocitySmoothingCount - SimTicks, 0);
		}
	}

	Chaos::FVec3 ErrorVelocitySmoothingV = Chaos::FVec3::Zero();
	Chaos::FVec3 ErrorVelocitySmoothingX = Chaos::FVec3::Zero();
	Chaos::FVec3 ErrorVelocitySmoothingXPrev = Chaos::FVec3::Zero();
	int32 ErrorVelocitySmoothingCount = 0;
};
