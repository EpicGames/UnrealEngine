// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Framework/MultiBufferResource.h"
#include "Chaos/Matrix.h"
#include "Misc/ScopeLock.h"
#include "ChaosLog.h"
#include "Chaos/Framework/PhysicsProxyBase.h"
#include "Framework/Threading.h"
#include "Chaos/ParticleDirtyFlags.h"
#include "Async/ParallelFor.h"
#include "Containers/Queue.h"
#include "Chaos/ChaosMarshallingManager.h"
#include "Chaos/AsyncInitBodyHelper.h"
#include "Stats/Stats.h"
#include "ChaosSolversModule.h"

#if WITH_CHAOS_VISUAL_DEBUGGER
#include "ChaosVisualDebugger/ChaosVDContextProvider.h"
#endif

#include "ChaosDebugDraw/ChaosDDTypes.h"
#include "Chaos/ChaosDebugNameDefines.h"

class FChaosSolversModule;
class FPhysicsReplicationAsync;
class IPhysicsReplicationAsync;
class IPhysicsReplicationLODAsync;

DECLARE_MULTICAST_DELEGATE_OneParam(FSolverPreAdvance, Chaos::FReal);
DECLARE_MULTICAST_DELEGATE_OneParam(FSolverPreBuffer, Chaos::FReal);
DECLARE_MULTICAST_DELEGATE_OneParam(FSolverPostAdvance, Chaos::FReal);
DECLARE_MULTICAST_DELEGATE(FSolverTeardown);

#ifndef UE_CHAOS_CALLBACK_TRACESTATS
#define UE_CHAOS_CALLBACK_TRACESTATS STATS && !UE_BUILD_SHIPPING
#endif

DECLARE_CYCLE_STAT_EXTERN(TEXT("Async Pull Results"), STAT_AsyncPullResults, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Async Interpolate Results"), STAT_AsyncInterpolateResults, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Sync Pull Results"), STAT_SyncPullResults, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Process Single Particle Proxies"), STAT_ProcessSingleProxy, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Process Geometry Collections Proxies"), STAT_ProcessGCProxy, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Process Cluster Union Proxies"), STAT_ProcessClusterUnionProxy, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Pull Constraints"), STAT_PullConstraints, STATGROUP_Chaos, CHAOS_API);

namespace Chaos
{
	class FPhysicsSolverBase;
	struct FPendingSpatialDataQueue;
	class FChaosResultsManager;
	class FRewindData;
	class IRewindCallback;
	enum EPendingSpatialDataOperation : uint8;

	extern CHAOS_API int32 GSingleThreadedPhysics;
	extern CHAOS_API int32 UseAsyncInterpolation;
	extern CHAOS_API int32 ForceDisableAsyncPhysics;
	extern CHAOS_API FRealSingle AsyncInterpolationMultiplier;
	extern CHAOS_API int32 RewindBeforeAdvance;

	struct FSubStepInfo
	{
		FSubStepInfo()
			: PseudoFraction(1.0)
			, Step(1)
			, NumSteps(1)
			, bSolverSubstepped(false)
		{
		}

		FSubStepInfo(const FReal InPseudoFraction, const int32 InStep, const int32 InNumSteps)
			: PseudoFraction(InPseudoFraction)
			, Step(InStep)
			, NumSteps(InNumSteps)
			, bSolverSubstepped(false)
		{

		}

		FSubStepInfo(const FReal InPseudoFraction, const int32 InStep, const int32 InNumSteps, bool bInSolverSubstepped)
			: PseudoFraction(InPseudoFraction)
			, Step(InStep)
			, NumSteps(InNumSteps)
			, bSolverSubstepped(bInSolverSubstepped)
		{

		}

		//This is NOT Step / NumSteps, this is to allow for kinematic target interpolation which uses its own logic
		FReal PseudoFraction;
		int32 Step;
		int32 NumSteps;
		bool bSolverSubstepped;
	};


	enum EAsyncBlockMode
	{
		BlockOnlyPastFrames = 0,
		BlockForBestInterpolation = 1,
		DoNoBlock = 2
	};

	/**
	 * Task responsible for processing the command buffer of a single solver and preparing data before solver task and callbacks are run
	 */
	class FPhysicsSolverProcessPushDataTask
	{
	public:

		FPhysicsSolverProcessPushDataTask(FPhysicsSolverBase& InSolver, FPushPhysicsData* InPushData)
			: Solver(InSolver)
			, PushData(InPushData){}

		TStatId GetStatId() const { RETURN_QUICK_DECLARE_CYCLE_STAT(FPhysicsSolverProcessPushDataTask, STATGROUP_TaskGraphTasks); }
		static CHAOS_API ENamedThreads::Type GetDesiredThread();
		static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }
		void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent) { ProcessPushData(); }
		CHAOS_API void ProcessPushData();

	private:

		FPhysicsSolverBase& Solver;
		FPushPhysicsData* PushData;
	};

	/**
	 * Task responsible for triggering any game thread callbacks before solver can advance (not scheduled if none are registered)
	 */
	class FPhysicsSolverFrozenGTPreSimCallbacks
	{
	public:

		FPhysicsSolverFrozenGTPreSimCallbacks(FPhysicsSolverBase& InSolver) : Solver(InSolver) {}

		TStatId GetStatId() const { RETURN_QUICK_DECLARE_CYCLE_STAT(FPhysicsSolverFrozenGTPreSimCallbacks, STATGROUP_TaskGraphTasks); }
		static ENamedThreads::Type GetDesiredThread() { return ENamedThreads::SetTaskPriority(ENamedThreads::GameThread, ENamedThreads::HighTaskPriority); }
		static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }
		void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent) { GTPreSimCallbacks(); }
		CHAOS_API void GTPreSimCallbacks();

	private:

		FPhysicsSolverBase& Solver;
	};

	/**
	 * Task responsible for advancing the solver once data has been prepared and GT callbacks have fired
	 */
	class FPhysicsSolverAdvanceTask
	{
	public:

		CHAOS_API FPhysicsSolverAdvanceTask(FPhysicsSolverBase& InSolver, FPushPhysicsData* InPushData);
		
		TStatId GetStatId() const { RETURN_QUICK_DECLARE_CYCLE_STAT(FPhysicsSolverAdvanceTask, STATGROUP_TaskGraphTasks); }
		static CHAOS_API ENamedThreads::Type GetDesiredThread();
		static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }
		CHAOS_API void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent);
		CHAOS_API void AdvanceSolver();

	private:

		FPhysicsSolverBase& Solver;
		FPushPhysicsData* PushData;

#if WITH_CHAOS_VISUAL_DEBUGGER
		FChaosVDContext CVDContext;
	public:
		FChaosVDContext& GetChaosVDContextData()
		{
			return CVDContext;
		};
#endif
	};


	/**
	 * Task responsible for rewinding the solver if required by networked physics
	 */
	class FPhysicsSolverRewindTask
	{
	public:

		FPhysicsSolverRewindTask(FPhysicsSolverBase& InSolver, FPushPhysicsData* InPushData) : Solver(InSolver), PushData(InPushData) {}

		TStatId GetStatId() const { RETURN_QUICK_DECLARE_CYCLE_STAT(FPhysicsSolverRewindTask, STATGROUP_TaskGraphTasks); }
		static CHAOS_API ENamedThreads::Type GetDesiredThread();
		static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }
		void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent) { RewindSolver(); }
		void RewindSolver();

	private:

		FPhysicsSolverBase& Solver;
		FPushPhysicsData* PushData;
	};

	// Container for all steps required to fully update a solver
	class FAllSolverTasks
	{
	public:

		FAllSolverTasks(FPhysicsSolverBase& InSolver, FPushPhysicsData* PushData)
			: ProcessPushData(InSolver, PushData)
			, GTPreSimCallbacks(InSolver)
			, AdvanceTask(InSolver, PushData)
			, RewindTask(InSolver, PushData)
			, Solver(InSolver)
		{
		}

		FPhysicsSolverProcessPushDataTask ProcessPushData;
		FPhysicsSolverFrozenGTPreSimCallbacks GTPreSimCallbacks;
		FPhysicsSolverAdvanceTask AdvanceTask;
		FPhysicsSolverRewindTask RewindTask;

		CHAOS_API void AdvanceSolver();

	private:

		FPhysicsSolverBase& Solver;
	};

	// Container for all physics-thread steps required to update a solver
	// This leaves out the game thread callbacks for situations that require and update only in a physics-thread context
	class FSolverTasksPTOnly
	{
	public:
		FSolverTasksPTOnly() = delete;
		FSolverTasksPTOnly(const FSolverTasksPTOnly&) = delete;
		FSolverTasksPTOnly(FSolverTasksPTOnly&&) = delete;
		FSolverTasksPTOnly& operator=(const FSolverTasksPTOnly&) = delete;
		FSolverTasksPTOnly& operator=(FSolverTasksPTOnly&&) = delete;

		FSolverTasksPTOnly(FPhysicsSolverBase& InSolver, FPushPhysicsData* InPushData)
			: ProcessPushData(InSolver, InPushData)
			, AdvanceTask(InSolver, InPushData)
			, RewindTask(InSolver, InPushData)
			, Solver(InSolver)
		{}
	
		CHAOS_API void AdvanceSolver();

	private:

		FPhysicsSolverProcessPushDataTask ProcessPushData;
		FPhysicsSolverAdvanceTask AdvanceTask;
		FPhysicsSolverRewindTask RewindTask;

		FPhysicsSolverBase& Solver;
	};

	class FPersistentPhysicsTask;

	enum class ELockType: uint8;

	//todo: once refactor is done use just one enum
	enum class EThreadingModeTemp: uint8
	{
		DedicatedThread,
		TaskGraph,
		SingleThread
	};
	
	/** Base solver class storing events that will be used by the derived solver during the solve.
	 * Currently used by caching but could be used for any other pre/post solve works that need to be done.  */
	class FPhysicsSolverEvents
	{

	public:

		/** Only allow construction with valid parameters as well as restricting to module construction */
		virtual ~FPhysicsSolverEvents()
		{ 
			EventTeardown.Broadcast();
			ClearCallbacks(); 
		}
		
		/** Events */
		/** WARNING: Events are not threadsafe!*/
		/** Pre advance is called before any physics processing or simulation happens in a given physics update */
		CHAOS_API FDelegateHandle AddPreAdvanceCallback(FSolverPreAdvance::FDelegate InDelegate);
		CHAOS_API bool            RemovePreAdvanceCallback(FDelegateHandle InHandle);

		/** Pre buffer happens after the simulation has been advanced (particle positions etc. will have been updated) but GT results haven't been prepared yet */
		CHAOS_API FDelegateHandle AddPreBufferCallback(FSolverPreAdvance::FDelegate InDelegate);
		CHAOS_API bool            RemovePreBufferCallback(FDelegateHandle InHandle);

		/** Post advance happens after all processing and results generation has been completed */
		CHAOS_API FDelegateHandle AddPostAdvanceCallback(FSolverPostAdvance::FDelegate InDelegate);
		CHAOS_API bool            RemovePostAdvanceCallback(FDelegateHandle InHandle);

		/** Teardown happens as the solver is destroyed or streamed out */
		CHAOS_API FDelegateHandle AddTeardownCallback(FSolverTeardown::FDelegate InDelegate);
		CHAOS_API bool            RemoveTeardownCallback(FDelegateHandle InHandle);

		/** Clear all the callbacks*/
		void            ClearCallbacks() 
		{
			EventPreSolve.Clear(); 
			EventPreBuffer.Clear(); 
			EventPostSolve.Clear(); 
			EventTeardown.Clear();
		}
	
	protected:
			
		/** Storage for events, see the Add/Remove pairs above for event timings */
		FSolverPreAdvance EventPreSolve;
		FSolverPreBuffer EventPreBuffer;
		FSolverPostAdvance EventPostSolve;
		FSolverTeardown EventTeardown;
	};

#if UE_CHAOS_CALLBACK_TRACESTATS
	class FScopedTraceSolverCallback : public FCycleCounter
	{
	public:
		FScopedTraceSolverCallback(ISimCallbackObject* InCallback)
		{
			if(InCallback)
			{
				if(TStatId CallbackStatId = InCallback->GetStatId(); FThreadStats::IsCollectingData(CallbackStatId))
				{
					Start(CallbackStatId);
				}
			}
		}

		~FScopedTraceSolverCallback()
		{
			Stop();
		}
	};
#else
	class FScopedTraceSolverCallback
	{
	public:
		FScopedTraceSolverCallback(ISimCallbackObject* InCallback)
		{}
	};
#endif

	class FPhysicsSolverBase : public FPhysicsSolverEvents
	{
	public:

		template <typename Lambda>
		void CastHelper(const Lambda& Func)
		{
			Func((FPBDRigidsSolver&)*this);
		}

		FPBDRigidsSolver& CastChecked()
		{
			return (FPBDRigidsSolver&)(*this);
		}

		CHAOS_API void ChangeBufferMode(EMultiBufferMode InBufferMode);

		void AddDirtyProxy(IPhysicsProxyBase * ProxyBaseIn)
		{
			check(ProxyBaseIn->GetMarkedDeleted() == false);
			MarshallingManager.AddDirtyProxy(ProxyBaseIn);
		}
		void RemoveDirtyProxy(IPhysicsProxyBase * ProxyBaseIn)
		{
			MarshallingManager.RemoveDirtyProxy(ProxyBaseIn);
		}

		void RemoveDirtyProxyFromHistory_Internal(IPhysicsProxyBase* ProxyBaseIn)
		{
			MarshallingManager.RemoveDirtyProxyFromHistory_Internal(ProxyBaseIn);
		}

		void RemoveDirtyProxyIfNoShapesAreDirty(IPhysicsProxyBase* ProxyBaseIn)
		{
			MarshallingManager.RemoveDirtyProxyIfNoShapesAreDirty(ProxyBaseIn);
		}

		const FDirtyProxiesBucketInfo& GetDirtyProxyBucketInfo_External()
		{
			return MarshallingManager.GetDirtyProxyBucketInfo_External();
		}

		int32 GetDirtyProxyBucketInfoNum_External(EPhysicsProxyType Type)
		{
			return MarshallingManager.GetDirtyProxyBucketInfoNum_External(Type);
		}

		// Batch dirty proxies without checking DirtyIdx.
		template <typename TProxiesArray>
		void AddDirtyProxiesUnsafe(TProxiesArray& ProxiesArray)
		{
			MarshallingManager.AddDirtyProxiesUnsafe(ProxiesArray);
		}

		void AddDirtyProxyShape(IPhysicsProxyBase* ProxyBaseIn, int32 ShapeIdx)
		{
			MarshallingManager.AddDirtyProxyShape(ProxyBaseIn, ShapeIdx);
		}

		void SetNumDirtyShapes(IPhysicsProxyBase* Proxy, int32 NumShapes)
		{
			MarshallingManager.SetNumDirtyShapes(Proxy, NumShapes);
		}

		/** Creates a new sim callback object of the type given. Caller expected to free using FreeSimCallbackObject_External*/
		template <typename TSimCallbackObjectType, typename... ConstructorArgTypes>
		inline TSimCallbackObjectType* CreateAndRegisterSimCallbackObject_External(ConstructorArgTypes&&... ConstructorArgs)
		{
			auto NewCallbackObject = new TSimCallbackObjectType(Forward<ConstructorArgTypes>(ConstructorArgs)...);
			//If at least one callback is on frozen GT we mark it as such
			//Note this is never cleaned up (to avoid race conditions of register/unregister).
			//Expectation is that there's typically just one of these on frozen gt that triggers all the fixed tick on gt (maybe two if another system wants direct control)
			//This means we don't expect to optimize for the case where a gt callback goes away half way through sim
			if (NewCallbackObject->HasOption(ESimCallbackOptions::RunOnFrozenGameThread))
			{
				bSolverHasFrozenGameThreadCallbacks = true;
			}

			RegisterSimCallbackObject_External(NewCallbackObject);
			NewCallbackObject->PostInitialize_External();
			return NewCallbackObject;
		}

		template <typename TSimCallbackObjectType>
		UE_DEPRECATED(5.1, "Use version with no arguments, and provide TOptions bitmask parameter to TSimCallbackObject instead.")
		TSimCallbackObjectType* CreateAndRegisterSimCallbackObject_External(bool bContactModification, bool bRegisterRewindCallback = false)
		{
			TSimCallbackObjectType* CallbackObject = CreateAndRegisterSimCallbackObject_External<TSimCallbackObjectType>();
			ESimCallbackOptions& Options = CallbackObject->Options;

			if (bContactModification) {
				Options |= ESimCallbackOptions::ContactModification;
			} else {
				Options &= ~ESimCallbackOptions::ContactModification;
			}

			if (bRegisterRewindCallback) {
				Options |= ESimCallbackOptions::Rewind;
			} else {
				Options &= ~ESimCallbackOptions::Rewind;
			}

			return CallbackObject;
		}

		CHAOS_API void EnqueueSimcallbackRewindRegisteration(ISimCallbackObject* Callback);

		void UnregisterAndFreeSimCallbackObject_External(ISimCallbackObject* SimCallbackObject)
		{
			SimCallbackObject->PreShutdown_External();
			MarshallingManager.UnregisterSimCallbackObject_External(SimCallbackObject);

			// Unregister external callback
			for (int32 Idx = InjectInputsWatchers_External.Num() - 1; Idx >= 0; --Idx)
			{
				if (InjectInputsWatchers_External[Idx]->bPendingDelete_External)
				{
					InjectInputsWatchers_External.RemoveAtSwap(Idx, EAllowShrinking::No);
				}
			}
		}

		template <typename Lambda>
		void RegisterSimOneShotCallback(Lambda&& Func)
		{
			//do we need a pool to avoid allocations?
			auto CommandObject = new FSimCallbackCommandObject(MoveTemp(Func));
			MarshallingManager.RegisterSimCommand_External(CommandObject);
		}

		template <typename Lambda>
		void EnqueueCommandImmediate(Lambda&& Func)
		{
			//TODO: remove this check. Need to rename with _External
			check(Chaos::CVars::bEnableAsyncInitBody || IsInGameThread());
			RegisterSimOneShotCallback(MoveTemp(Func));
		}

		/** Enqueue a lambda function (command) to run on the physics thread on a specific physics step
		* The command will be reapplied during physics resimulation on the same step */
		template <typename Lambda>
		void EnqueueCommandScheduled_External(const int32 ExecuteOnStep, Lambda&& Func)
		{
			FSimCallbackCommandObject* CommandObject = new FSimCallbackCommandObject(MoveTemp(Func), ExecuteOnStep);
			MarshallingManager.RegisterSimCommand_External(CommandObject);
		}

		CHAOS_API void SetRewindCallback(TUniquePtr<IRewindCallback>&& RewindCallback);

		FRewindData* GetRewindData()
		{
			return MRewindData.Get();
		}

		const FRewindData* GetRewindData() const
		{
			return MRewindData.Get();
		}

		IRewindCallback* GetRewindCallback()
		{
			return MRewindCallback.Get();
		}

		const IRewindCallback* GetRewindCallback() const
		{
			return MRewindCallback.Get();
		}

		bool ShouldApplyRewindCallbacks()
		{
			return MRewindCallback.IsValid() && MRewindData.IsValid();
		}

		/** Enable or disable an additional resim cache based on IResimCacheBase (FEvolutionResimCache by default) that also caches particle collision constraints
		* this is disabled by default since cached collisions will be wrong during a resimulation for particles that are desynced
		* and particles that are in sync doesn't need collision data since we just step through their states in history." */
		void SetUseCollisionResimCache(bool InUseCollisionResimCache)
		{
			bUseCollisionResimCache = InUseCollisionResimCache;
		}

		/** Check if an additional resim cache based on IResimCacheBase (FEvolutionResimCache by default) is being used. Read FPhysicsSolverBase.SetUseCollisionResimCache() for more info. */
		bool GetUseCollisionResimCache() const
		{
			return bUseCollisionResimCache;
		}

		void SetPhysicsReplication_Internal(IPhysicsReplicationAsync* InPhysicsReplication)
		{
			PhysicsReplicationAsync = InPhysicsReplication;
		}

		IPhysicsReplicationAsync* GetPhysicsReplication_Internal()
		{
			return PhysicsReplicationAsync;
		}

		const IPhysicsReplicationAsync* GetPhysicsReplication_Internal() const
		{
			return PhysicsReplicationAsync;
		}

		void SetPhysicsReplicationLOD_Internal(IPhysicsReplicationLODAsync* InPhysicsReplicationLOD)
		{
			PhysicsReplicationLODAsync = InPhysicsReplicationLOD;
		}

		IPhysicsReplicationLODAsync* GetPhysicsReplicationLOD_Internal()
		{
			return PhysicsReplicationLODAsync;
		}

		//Used as helper for GT to go from unique idx back to gt particle
		//If GT deletes a particle, this function will return null (that's a good thing when consuming async outputs as GT may have already deleted the particle we care about)
		//Note: if the physics solver has been advanced after the particle was freed on GT, the index may have been freed and reused.
		//In this case instead of getting a nullptr you will get an unrelated (wrong) GT particle
		//Because of this we keep the index alive for as long as the async callback can lag behind. This way as long as you immediately consume the output, you will always be sure the unique index was not released.
		//Practically the flow should always be like this:
		//advance the solver and trigger callbacks. Callbacks write to outputs. Consume the outputs on GT and use this function _before_ advancing solver again
		FGeometryParticle* UniqueIdxToGTParticle_External(const FUniqueIdx& UniqueIdx) const
		{
			FGeometryParticle* Result = nullptr;
			if (ensure(UniqueIdx.Idx < UniqueIdxToGTParticles.Num()))	//asking for particle on index that has never been allocated
			{
				Result = UniqueIdxToGTParticles[UniqueIdx.Idx];
			}

			return Result;
		}

		//Ensures that any running tasks finish.
		void WaitOnPendingTasks_External()
		{
			if(PendingTasks && !PendingTasks->IsComplete())
			{
				FTaskGraphInterface::Get().WaitUntilTaskCompletes(PendingTasks);
			}
		}

		virtual void KillSafeAsyncTasks()
		{
		}

		virtual bool AreAnyTasksPending() const
		{
			return false;
		}

		bool IsPendingTasksComplete() const
		{
			if (PendingTasks && !PendingTasks->IsComplete())
			{
				return false;
			}

			return true;
		}

		const UObject* GetOwner() const
		{ 
			return Owner; 
		}

		void SetOwner(const UObject* InOwner)
		{
			Owner = InOwner;
		}

		void SetThreadingMode_External(EThreadingModeTemp InThreadingMode)
		{
			if (!!GSingleThreadedPhysics) { InThreadingMode = EThreadingModeTemp::SingleThread; }

			if(InThreadingMode != ThreadingMode)
			{
				if(InThreadingMode == EThreadingModeTemp::SingleThread)
				{
					WaitOnPendingTasks_External();
				}
				ThreadingMode = InThreadingMode;
			}
		}
		
		void MarkShuttingDown()
		{
			bIsShuttingDown = true;
		}

		bool IsShuttingDown() const { return bIsShuttingDown; }

		CHAOS_API void EnableAsyncMode(FReal FixedDt);
		
		FReal GetAsyncDeltaTime() const { return AsyncDt; }
		void SetAsyncDeltaTime(FReal InDt) { AsyncDt = InDt; }
		
		CHAOS_API void DisableAsyncMode();
		
		virtual void AddResimulationRequest_Internal(FPushPhysicsData& PushData);
		virtual void ConditionalApplyRewind_Internal(){}
		virtual bool IsResimming() const {return false;}

#if WITH_CHAOS_VISUAL_DEBUGGER
		virtual int32 GetCVDFrameNumber() const { return INDEX_NONE; }
#endif

		FChaosMarshallingManager& GetMarshallingManager() { return MarshallingManager; }
		FChaosResultsManager& GetResultsManager() { return *PullResultsManager; }

		EThreadingModeTemp GetThreadingMode() const
		{
			return ThreadingMode;
		}

		CHAOS_API FGraphEventRef AdvanceAndDispatch_External(FReal InDt);
 
#if CHAOS_SOLVER_DEBUG_NAME
		CHAOS_API void SetDebugName(const FName& Name);
#endif
		CHAOS_API FName GetDebugName() const;


		//Tells us if we're on the frozen game thread. This is needed for knowing which data to read/write to
		//The IsInGameThread check is so that other threads (e.g audio thread) which might be running queries in parallel will continue to use the correct interpolated GT data
		bool IsGameThreadFrozen() const { return bGameThreadFrozen; }

		void SetGameThreadFrozen(bool InGameThreadFrozen)
		{
			check(IsInGameThread());
			bGameThreadFrozen = InGameThreadFrozen;
		}

		virtual void ApplyCallbacks_Internal();

		void FinalizeCallbackData_Internal()
		{
			for (ISimCallbackObject* Callback : SimCallbackObjects)
			{
				Callback->FinalizeOutputData_Internal();
				Callback->SetCurrentInput_Internal(nullptr);
			}
		}

		CHAOS_API void UpdateParticleInAccelerationStructure_External(FGeometryParticle* Particle, EPendingSpatialDataOperation InOperation);

		bool IsPaused_External() const
		{
			return bPaused_External;
		}

		void SetIsPaused_External(bool bShouldPause)
		{
			bPaused_External = bShouldPause;
		}

		/** Used to update external thread data structures. Include PhysicsSolverBaseImpl.h to call this function*/
		template<typename TDispatcher>
		void PullPhysicsStateForEachDirtyProxy_External(TDispatcher& Dispatcher);

		template <typename RigidLambda, typename ConstraintLambda, typename GeometryCollectionLambda>
		UE_DEPRECATED(5.3, "Use PullPhysicsStateForEachDirtyProxy_External with the TDispatcher template parameter instead.")
		void PullPhysicsStateForEachDirtyProxy_External(const RigidLambda& RigidFunc, const ConstraintLambda& ConstraintFunc, const GeometryCollectionLambda& GeometryCollectionFunc);

		template <typename RigidLambda, typename ConstraintLambda>
		UE_DEPRECATED(5.3, "Use PullPhysicsStateForEachDirtyProxy_External with the TDispatcher template parameter instead.")
		void PullPhysicsStateForEachDirtyProxy_External(const RigidLambda& RigidFunc, const ConstraintLambda& ConstraintFunc);

		bool IsUsingAsyncResults() const
		{
			return !ForceDisableAsyncPhysics && AsyncDt >= 0;
		}

		bool IsUsingFixedDt() const
		{
			return IsUsingAsyncResults() && UseAsyncInterpolation;
		}

		void SetMaxDeltaTime_External(float InMaxDeltaTime)
		{
			MMaxDeltaTime = InMaxDeltaTime;
		}

		void SetMinDeltaTime_External(float InMinDeltaTime)
		{
			// a value approaching zero is not valid/desired
			MMinDeltaTime = FMath::Max(InMinDeltaTime, UE_SMALL_NUMBER);
		}

		float GetMaxDeltaTime_External() const
		{
			return MMaxDeltaTime;
		}

		float GetMinDeltaTime_External() const
		{
			return MMinDeltaTime;
		}

		void SetMaxSubSteps_External(const int32 InMaxSubSteps)
		{
			MMaxSubSteps = InMaxSubSteps;
		}

		int32 GetMaxSubSteps_External() const
		{
			return MMaxSubSteps;
		}

		void SetSolverSubstep_External(bool bInSubstepExternal)
		{
			bSolverSubstep_External = bInSubstepExternal;
		}

		bool GetSolverSubstep_External() const
		{
			return bSolverSubstep_External;
		}

		FReal GetAccumulatedTime() const
		{
			return AccumulatedTime;
		}

		/** Returns the time used by physics results. If fixed dt is used this will be the interpolated time */
		FReal GetPhysicsResultsTime_External() const
		{
			const FReal ExternalTime = MarshallingManager.GetExternalTime_External() + AccumulatedTime;
			if (IsUsingFixedDt())
			{
				//fixed dt uses interpolation and looks into the past
				return ExternalTime - AsyncDt * AsyncMultiplier;
			}
			else
			{
				return ExternalTime;
			}
		}

		virtual void FlipEventManagerBuffer() {}

		/**/
		void SetSolverTime(const FReal InTime) { MTime = InTime; }
		const FReal GetSolverTime() const { return MTime; }


		/**/
		FReal GetLastDt() const { return MLastDt; }

		/** 
		  Set the Async Block Mode, valid mode can be 0, 1, or 2
		  0 blocks on any physics steps generated from past GT Frames, and blocks on none of the tasks from current frame.
		  1 blocks on everything except the single most recent task (including tasks from current frame)
		  1 should guarantee we will always have a future output for interpolation from 2 frames in the past
		  2 doesn't block the game thread. Physics steps could be eventually be dropped if taking too much time.
		*/
		void SetAsyncPhysicsBlockMode(EAsyncBlockMode InAsyncBlockMode) { AsyncBlockMode = InAsyncBlockMode; }

		/** 
		* Set the async interpolation multiplier which is how many multiples of the fixed dt should we look behind for interpolation.
		*/
		void SetAsyncInterpolationMultiplier(FRealSingle InAsyncInterpolationMultiplier) { AsyncMultiplier = InAsyncInterpolationMultiplier; }

		float GetAsyncInterpolationMultiplier() const { return AsyncMultiplier; }

		/** Check if we can enable debugging informations for network physics */
		CHAOS_API static bool CanDebugNetworkPhysicsPrediction();

		/** Check if network physics is enables or not */
		static bool IsNetworkPhysicsPredictionEnabled()
		{
			const bool NetworkPhysicsEnabled = FChaosSolversModule::GetModule()->GetSettingsProvider().GetPhysicsPredictionEnabled();
			return NetworkPhysicsEnabled;
		}

		/** Get the time length to cache physics history for, based on Project Settings -> Physics -> Physics Prediction -> MaxSupportedLatencyPrediction */
		static float GetPhysicsHistoryTimeLength()
		{
			const float PhysicsHistoryTimeLength = FChaosSolversModule::GetModule()->GetSettingsProvider().GetPhysicsHistoryTimeLength();
			return PhysicsHistoryTimeLength;
		}

		/** Get the number of physics history frames to cache */
		static int32 GetPhysicsHistoryCount()
		{
			const int32 PhysicsHistoryCount = FChaosSolversModule::GetModule()->GetSettingsProvider().GetPhysicsHistoryCount();
			return PhysicsHistoryCount;
		}

		static bool GetResimulationErrorPositionThresholdEnabled()
		{
			const bool ResimulationErrorPositionThresholdEnabled = FChaosSolversModule::GetModule()->GetSettingsProvider().GetResimulationErrorPositionThresholdEnabled();
			return ResimulationErrorPositionThresholdEnabled;
		}

		static float GetResimulationErrorPositionThreshold()
		{
			const float ResimulationErrorPositionThreshold = FChaosSolversModule::GetModule()->GetSettingsProvider().GetResimulationErrorPositionThreshold();
			return ResimulationErrorPositionThreshold;
		}

		static bool GetResimulationErrorRotationThresholdEnabled()
		{
			const bool ResimulationErrorRotationThresholdEnabled = FChaosSolversModule::GetModule()->GetSettingsProvider().GetResimulationErrorRotationThresholdEnabled();
			return ResimulationErrorRotationThresholdEnabled;
		}

		static float GetResimulationErrorRotationThreshold()
		{
			const float ResimulationErrorRotationThreshold = FChaosSolversModule::GetModule()->GetSettingsProvider().GetResimulationErrorRotationThreshold();
			return ResimulationErrorRotationThreshold;
		}

		static bool GetResimulationErrorLinearVelocityThresholdEnabled()
		{
			const bool ResimulationErrorLinearVelocityThresholdEnabled = FChaosSolversModule::GetModule()->GetSettingsProvider().GetResimulationErrorLinearVelocityThresholdEnabled();
			return ResimulationErrorLinearVelocityThresholdEnabled;
		}

		static float GetResimulationErrorLinearVelocityThreshold()
		{
			const float ResimulationErrorLinearVelocityThreshold = FChaosSolversModule::GetModule()->GetSettingsProvider().GetResimulationErrorLinearVelocityThreshold();
			return ResimulationErrorLinearVelocityThreshold;
		}

		static bool GetResimulationErrorAngularVelocityThresholdEnabled()
		{
			const bool ResimulationErrorAngularVelocityThresholdEnabled = FChaosSolversModule::GetModule()->GetSettingsProvider().GetResimulationErrorAngularVelocityThresholdEnabled();
			return ResimulationErrorAngularVelocityThresholdEnabled;
		}

		static float GetResimulationErrorAngularVelocityThreshold()
		{
			const float ResimulationErrorAngularVelocityThreshold = FChaosSolversModule::GetModule()->GetSettingsProvider().GetResimulationErrorAngularVelocityThreshold();
			return ResimulationErrorAngularVelocityThreshold;
		}

		/** Return the interpolation lerp in case the resim is off */
		CHAOS_API static float NetworkPhysicsInterpolationLerp();

		/** Get the standalone solver flag */
		bool IsStandaloneSolver() const { return bIsStandaloneSolver;}
		
		/** Set the standalone solver flag */
		void SetStandaloneSolver(const bool bStandaloneSolver) {bIsStandaloneSolver = bStandaloneSolver;}

	protected:

		/** Boolean to check if the solver is a standalone solver responsible to spawn its own task */
		bool bIsStandaloneSolver = false;
		
		/** Mode that the results buffers should be set to (single, double, triple) */
		EMultiBufferMode BufferMode;
		
		EThreadingModeTemp ThreadingMode;

		/** Protected construction so callers still have to go through the module to create new instances */
		CHAOS_API FPhysicsSolverBase(const EMultiBufferMode BufferingModeIn,const EThreadingModeTemp InThreadingMode,UObject* InOwner, FReal AsyncDt);

		/** Only allow construction with valid parameters as well as restricting to module construction */
		CHAOS_API virtual ~FPhysicsSolverBase();

		static CHAOS_API void DestroySolver(FPhysicsSolverBase& InSolver);

		FPhysicsSolverBase() = delete;
		FPhysicsSolverBase(const FPhysicsSolverBase& InCopy) = delete;
		FPhysicsSolverBase(FPhysicsSolverBase&& InSteal) = delete;
		FPhysicsSolverBase& operator =(const FPhysicsSolverBase& InCopy) = delete;
		FPhysicsSolverBase& operator =(FPhysicsSolverBase&& InSteal) = delete;

		virtual void PrepareAdvanceBy(const FReal Dt) = 0;
		virtual void AdvanceSolverBy(const FSubStepInfo& SubStepInfo = FSubStepInfo()) = 0;
		virtual void PushPhysicsState(const FReal Dt, const int32 NumSteps, const int32 NumExternalSteps) = 0;
		virtual void ProcessPushedData_Internal(FPushPhysicsData& PushDataArray) = 0;
		virtual void SetExternalTimestampConsumed_Internal(const int32 Timestamp) = 0;

#if CHAOS_SOLVER_DEBUG_NAME
		virtual void OnDebugNameChanged() {}

		FName DebugName;
#endif

		FChaosMarshallingManager MarshallingManager;
		TUniquePtr<FChaosResultsManager> PullResultsManager;	//must come after MarshallingManager since it knows about MarshallingManager

		// The spatial operations not yet consumed by the internal sim. Use this to ensure any GT operations are seen immediately
		TUniquePtr<FPendingSpatialDataQueue> PendingSpatialOperations_External;

		TArray<ISimCallbackObject*> SimCallbackObjects;
		TArray<ISimCallbackObject*> MidPhaseModifiers;
		TArray<ISimCallbackObject*> CCDModifiers;
		TArray<ISimCallbackObject*> StrainModifiers;
		TArray<ISimCallbackObject*> ContactModifiers;
		TArray<ISimCallbackObject*> RegistrationWatchers;
		TArray<ISimCallbackObject*> UnregistrationWatchers;
		TArray<ISimCallbackObject*> InjectInputsWatchers_External;
		TArray<ISimCallbackObject*> ProcessInputsWatchers;
		TArray<ISimCallbackObject*> PhysicsObjectUnregistrationWatchers;

		TUniquePtr<FRewindData> MRewindData;
		TUniquePtr<IRewindCallback> MRewindCallback;
		IPhysicsReplicationAsync* PhysicsReplicationAsync = nullptr;
		IPhysicsReplicationLODAsync* PhysicsReplicationLODAsync = nullptr;

		bool bUseCollisionResimCache;

		FGraphEventRef PendingTasks;

		bool bSolverHasFrozenGameThreadCallbacks = false;
		bool bGameThreadFrozen = false;
		FReal MLastDt = FReal(0);
		FReal MTime = FReal(0);

	private:

		//This is private because the user should never create their own callback object
		//The lifetime management should always be done by solver to ensure callbacks are accessing valid memory on async tasks
		void RegisterSimCallbackObject_External(ISimCallbackObject* SimCallbackObject)
		{
			ensure(SimCallbackObject->Solver == nullptr);	//double register?
			SimCallbackObject->SetSolver_External(this);

			// Register external callback
			if (SimCallbackObject->HasOption(ESimCallbackOptions::InjectInputsExternal))
			{
				InjectInputsWatchers_External.Add(SimCallbackObject);
			}

			MarshallingManager.RegisterSimCallbackObject_External(SimCallbackObject);
		}

		// Number of ending solver task that has not been executed or that are still being executed
		TAtomic<int32> NumPendingSolverAdvanceTasks;

		/** 
		 * Whether this solver is paused. Paused solvers will still 'tick' however they will receive a Dt of zero so they can still
		 * build acceleration structures or accept inputs from external threads 
		 */
		bool bPaused_External;

		/** 
		 * Ptr to the engine object that is counted as the owner of this solver.
		 * Never used internally beyond how the solver is stored and accessed through the solver module.
		 * Nullptr owner means the solver is global or standalone.
		 * @see FChaosSolversModule::CreateSolver
		 */
		const UObject* Owner = nullptr;

		//TODO: why is this needed? seems bad to read from solver directly, should be buffered
		FRWLock SimMaterialLock;
		
		/** Scene lock object for external threads (non-physics) */
		TUniquePtr<FPhysSceneLock> ExternalDataLock_External;

#if UE_WITH_REMOTE_OBJECT_HANDLE
		/** Scene lock object for internal threads */
		TUniquePtr<FPhysSceneLock> InternalDataLock;
#endif

		friend FChaosSolversModule;
		friend FPhysicsSolverProcessPushDataTask;
		friend FPhysicsSolverAdvanceTask;
		friend FPhysicsSolverRewindTask;

		template<ELockType>
		friend struct TSolverSimMaterialScope;

		bool bIsShuttingDown;
		bool bSolverSubstep_External;
		FReal AsyncDt;
		FReal AccumulatedTime;
		float MMaxDeltaTime;
		float MMinDeltaTime;
		int32 MMaxSubSteps;
		int32 ExternalSteps;
		TArray<FGeometryParticle*> UniqueIdxToGTParticles;
		EAsyncBlockMode AsyncBlockMode;
		float AsyncMultiplier;

	public:

		/** Get the lock used for external data manipulation. A better API would be to use scoped locks so that getting a write lock is non-const */
		//NOTE: this is a const operation so that you can acquire a read lock on a const solver. The assumption is that non-const write operations are already marked non-const
		FPhysSceneLock& GetExternalDataLock_External() const { return *ExternalDataLock_External; }

#if UE_WITH_REMOTE_OBJECT_HANDLE
		UE_INTERNAL FPhysSceneLock& GetInternalDataLock() const
		{
			return *InternalDataLock;
		}
#endif

	protected:

		CHAOS_API void TrackGTParticle_External(FGeometryParticle& Particle);
		CHAOS_API void ClearGTParticle_External(FGeometryParticle& Particle);
		

#if !UE_BUILD_SHIPPING
	// Solver testing utility
	private:
		// instead of running advance task in single threaded, put in array for manual execution control for unit tests.
		bool bStealAdvanceTasksForTesting;
		TArray<FAllSolverTasks> StolenSolverAdvanceTasks;
	public:
		CHAOS_API void SetStealAdvanceTasks_ForTesting(bool bInStealAdvanceTasksForTesting);
		CHAOS_API void PopAndExecuteStolenAdvanceTask_ForTesting();
#endif

#if WITH_CHAOS_VISUAL_DEBUGGER
	private:
		FChaosVDContext CVDContextData;

	public:
		FChaosVDContext& GetChaosVDContextData()
		{
			return CVDContextData;
		};
#endif

#if CHAOS_DEBUG_DRAW
	public:
		virtual void SetDebugDrawScene(const ChaosDD::Private::FChaosDDScenePtr& InCDDScene) = 0;
#endif
	};
}


