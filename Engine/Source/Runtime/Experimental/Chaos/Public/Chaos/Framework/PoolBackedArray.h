// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/ObjectPool.h"

namespace Chaos::Private
{
	/**
	* An adapter to allow use of types with the TPoolBackedArray.
	*/
	template<typename T>
	struct TPoolBackedItemAdapter
	{
		// TPoolBackedArray requires the items to store their index in the array for fast removal
		static int32 GetArrayIndex(const T& Item) { return Item.GetArrayIndex(); }
		static void SetArrayIndex(T& Item, const int32 Index) { return Item.SetArrayIndex(Index); }

		// Called when an item is added to the free list.
		// Should perform work equivalent to a destructor but does not necessarily need to free up resources.
		static void TrashItem(T& Item) { Item.Trash(); }

		// Called when an item is created by recovering from the free list. 
		// Should perform work equivalent to a constructor.
		template<typename... TArgs>
		static void ReuseItem(T& Item, TArgs&&... Args) { Item.Reuse(Forward<TArgs>(Args)...); }
	};

	/**
	* An array of (pointers to) Items with items allocated from an ObjectPool.
	* Item pointers are persistent until freed.
	*
	* \tparam TItemAdapter Provides the Get/Set methods for array index of each item
	*/
	template<typename T, typename TItemAdapter = TPoolBackedItemAdapter<T>>
	class TPoolBackedArray
	{
	public:
		using FItem = T;
		using FItemPtr = FItem*;
		using FConstItemPtr = const FItem*;
		using FItemAdapter = TItemAdapter;
		using FRangedForIterator = typename TArray<FItemPtr>::RangedForIteratorType;
		using FRangedForConstIterator = typename TArray<FItemPtr>::RangedForConstIteratorType;

		TPoolBackedArray(const int32 NumItemsPerBlock)
			: Pool(NumItemsPerBlock)
		{
		}

		template<typename... TArgs>
		FItemPtr Alloc(TArgs&&... Args)
		{
			FItemPtr Item = nullptr;

			// Allocate from free list or pool
			if (!FreeItems.IsEmpty())
			{
				Item = FreeItems.Pop(EAllowShrinking::No);
				FItemAdapter::ReuseItem(*Item, Forward<TArgs>(Args)...);
			}
			else
			{
				Item = Pool.Alloc(Forward<TArgs>(Args)...);
			}

			// Add to the array and set the index
			const int32 Index = Items.Add(Item);
			FItemAdapter::SetArrayIndex(*Item, Index);

			return Item;
		}

		void Free(FItemPtr Item)
		{
			if (Item != nullptr)
			{
				// Remove item from the array using the index we set in Alloc
				const int32 Index = FItemAdapter::GetArrayIndex(*Item);
				check(Items[Index] == Item);
				Items.RemoveAtSwap(Index, EAllowShrinking::No);
				FItemAdapter::SetArrayIndex(*Item, INDEX_NONE);

				// Update the index of the item we swapped in
				if (Index < Items.Num())
				{
					FItemAdapter::SetArrayIndex(*Items[Index], Index);
				}

				// Move the item to the trash
				FItemAdapter::TrashItem(*Item);
				FreeItems.Add(Item);
			}
		}

		void Reset()
		{
			for (int32 Index = Num() - 1; Index >= 0; --Index)
			{
				Free(Items[Index]);
			}

			// Clear the free list
			for (FItemPtr Item : FreeItems)
			{
				Pool.Free(Item);
			}
			FreeItems.Reset();

			check(Items.Num() == 0);
			check(Pool.GetNumAllocated() == 0);
		}

		int32 Num() const
		{
			return Items.Num();
		}

		bool IsEmpty() const
		{
			return (Num() == 0);
		}

		void Reserve(const int32 Size)
		{
			Items.Reserve(Size);
			FreeItems.Reserve(Size);
			Pool.ReserveItems(Size);
		}

		FItemPtr operator[](const int32 Index)
		{
			return Items[Index];
		}

		const FItemPtr operator[](const int32 Index) const
		{
			return Items[Index];
		}

		FRangedForIterator begin()
		{
			return Items.begin();
		}

		FRangedForIterator end()
		{
			return Items.end();
		}

		FRangedForConstIterator begin() const
		{
			return Items.begin();
		}

		FRangedForConstIterator end() const
		{
			return Items.end();
		}

		void SortFreeLists()
		{
			FreeItems.Sort([](const FItem& L, const FItem& R) { return &L < &R; });

			//Pool.SortFreeLists();
		}

	private:

		TArray<FItemPtr> Items;
		TArray<FItemPtr> FreeItems;
		TObjectPool<FItem> Pool;
	};
}
