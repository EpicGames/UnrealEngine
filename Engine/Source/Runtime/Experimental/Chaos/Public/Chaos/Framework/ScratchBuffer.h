// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Containers/Array.h"
#include "Templates/AlignmentTemplates.h"
#include <type_traits>

#define CHAOS_SCRATCHBUFFER_CHECKSENTINEL (DO_CHECK)

namespace Chaos
{
	namespace Private
	{


		class FScratchBuffer
		{
		private:
			static const size_t SentinelValue = 0xA1B2C3D4A1B2C3D4ll;

		public:

			FScratchBuffer()
				: BufferNext(nullptr)
				, BufferBegin(nullptr)
				, BufferEnd(nullptr)
			{
			}

			~FScratchBuffer()
			{
				DestroyBuffer();
			}

			size_t BufferSize() const
			{
				// NOTE: this size does not include the sentinel
				return BufferEnd - BufferBegin;
			}

			void Empty()
			{
				DestroyBuffer();
			}

			void Reset(const size_t InMaxBytes)
			{
				if (InMaxBytes > BufferSize())
				{
					CreateBuffer(InMaxBytes);
				}
				BufferNext = BufferBegin;
			}

			template<typename T> T* AllocUninitialized()
			{
				static_assert(std::is_trivially_destructible_v<T>, "FScratchBuffer only supports trivially destructible types");

				void* Address = AllocAligned(sizeof(T), alignof(T));
				return (T*)Address;
			}

			template<typename T> T* AllocArrayUninitialized(const int32 Num)
			{
				static_assert(std::is_trivially_destructible_v<T>, "FScratchBuffer only supports trivially destructible types");

				const size_t AlignedSize = Align(sizeof(T), alignof(T));
				void* Address = AllocAligned(Num * AlignedSize, alignof(T));
				return (T*)Address;
			}

			template<typename T, typename... TArgs> T* Alloc(TArgs... Args)
			{
				T* Object = AllocUninitialized<T>();
				if (Object != nullptr)
				{
					new(Object) T(Args...);
				}
				return Object;
			}

			template<typename T, typename... TArgs> T* AllocArray(const int32 Num, TArgs... Args)
			{
				T* Objects = AllocArrayUninitialized<T>(Num);
				if (Objects != nullptr)
				{
					for (int32 Index = 0; Index < Num; ++Index)
					{
						new(&Objects[Index]) T(Args...);
					}
				}
				return Objects;
			}

		private:
			// Allocate some bytes with the specified size and slignment
			void* AllocAligned(const size_t InSize, const size_t InAlign)
			{
				uint8* const Address = Align(BufferNext, InAlign);

				uint8* const NewBufferNext = Address + InSize;
				if (NewBufferNext <= BufferEnd)
				{
					BufferNext = NewBufferNext;
					return Address;
				}

				return nullptr;
			}

			void CreateBuffer(const int32 InMaxBytes)
			{
				CheckSentinel();

				if (InMaxBytes != BufferSize())
				{
					DestroyBuffer();

					if (InMaxBytes > 0)
					{
						// Over-allocate and store a sentinel after the block
						BufferBegin = new uint8[InMaxBytes + sizeof(SentinelValue)];
					}

					if (BufferBegin != nullptr)
					{
						BufferEnd = BufferBegin + InMaxBytes;
					}

					InitSentinel();
				}

				BufferNext = BufferBegin;
			}

			void DestroyBuffer()
			{
				CheckSentinel();

				if (BufferBegin != nullptr)
				{
					delete[] BufferBegin;
				}

				BufferBegin = nullptr;
				BufferEnd = nullptr;
				BufferNext = nullptr;
			}

			size_t* Sentinel()
			{
				return (size_t*)BufferEnd;
			}

			void InitSentinel()
			{
				if (Sentinel() != nullptr)
				{
					*Sentinel() = SentinelValue;
				}
			}

			void CheckSentinel()
			{
#if CHAOS_SCRATCHBUFFER_CHECKSENTINEL
				if (Sentinel() != nullptr)
				{
					check(*Sentinel() == SentinelValue);
				}
#endif
			}

			uint8* BufferNext;
			uint8* BufferBegin;
			uint8* BufferEnd;
		};

	}
}
