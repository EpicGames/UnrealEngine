// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"

// If CHAOS_CHECK_UNCHECKED_ARRAY is 1, we still enable range checks on "unchecked" arrays.
// This is for debug builds or sanity checks in other builds.
#ifndef CHAOS_CHECK_UNCHECKED_ARRAY
#if UE_BUILD_DEBUG
#define CHAOS_CHECK_UNCHECKED_ARRAY 1
#else
#define CHAOS_CHECK_UNCHECKED_ARRAY 0
#endif
#endif

// @macro CHAOS_CARRAY_SENTINEL: Set to 1 to add some sentinels into TCArray to trap buffer overruns
#ifndef CHAOS_CARRAY_SENTINEL
#define CHAOS_CARRAY_SENTINEL 0
#endif

// @macro IF_CHAOS_CARRAY_SENTINEL(X): Include some code block X only when sentinels are enabled
// This is intended for one-line statements with no commas in them and is just shorthand for 
// #if CHAOS_CARRAY_SENTINEL
// X;
// #endif
#if CHAOS_CARRAY_SENTINEL
#define IF_CHAOS_CARRAY_SENTINEL(X) X
#else
#define IF_CHAOS_CARRAY_SENTINEL(X)
#endif

namespace Chaos
{
	/**
	 * @brief A c-style array of objects with non-shipping bounds checking.
	 * 
	 * This behaves exactly like a C-style array, although it also keeps track of the number of
	 * elements in the array. This element count is artficial - all elements in the array are 
	 * default constructed and will not be destructed until the array itself is destroyed.
	 * 
	 * @note the element type must have a default constructor
	*/
	template<typename T, int32 N>
	class TCArray
	{
	public:
		static const int32 MaxElements = N;
		using ElementType = T;

		static TCArray<T, N> MakeFull()
		{
			return TCArray<T, N>(N);
		}

		static TCArray<T, N> MakeEmpty()
		{
			return TCArray<T, N>(0);
		}

		inline TCArray() : NumElements(0)
		{
		}

		inline int32 Num() const
		{ 
			CheckSentinels();

			return NumElements;
		}

		inline int32 Max() const
		{ 
			CheckSentinels();
		
			return MaxElements; 
		}
		
		inline bool IsEmpty() const
		{ 
			CheckSentinels();

			return NumElements == 0;
		}
		
		inline bool IsFull() const
		{
			CheckSentinels();

			return NumElements == MaxElements;
		}

		inline ElementType& operator[](const int32 Index)
		{
			check(Index < NumElements);
			CheckSentinels();

			return Elements[Index];
		}

		inline const ElementType& operator[](const int32 Index) const
		{
			check(Index < NumElements);
			CheckSentinels();

			return Elements[Index];
		}

		/**
		 * @brief Set the number of elements in the array
		 * @note Does not reset any added or removed elements, it only changes the number-of-elements counter
		*/
		inline void SetNum(const int32 InNum)
		{
			check(InNum <= MaxElements);
			CheckSentinels();

			NumElements = InNum;
		}

		/**
		 * @brief Set the number of elements to 0
		 * @note Does not reset any elements
		*/
		inline void Reset()
		{
			CheckSentinels();

			NumElements = 0;
		}

		/**
		 * @brief Set the number of elements to 0
		 * @note Does not reset any elements
		*/
		inline void Empty()
		{
			CheckSentinels();

			NumElements = 0;
		}

		/**
		 * @brief Increase the size of the array without re-initializing the new element
		*/
		UE_DEPRECATED(5.5, "Renamed to AddUinitialized() to match TArray API")
		inline int32 Add()
		{
			return AddUninitialized();
		}

		/**
		 * @brief Increase the size of the array without re-initializing the new element
		*/
		inline int32 AddUninitialized()
		{
			check(NumElements < MaxElements);
			CheckSentinels();

			return NumElements++;
		}

		/**
		 * @brief Copy the element to the end of the array
		*/
		inline int32 Add(const ElementType& V)
		{
			check(NumElements < MaxElements);

			Elements[NumElements] = V;

			CheckSentinels();
			return NumElements++;
		}

		/**
		 * @brief Move the element to the end of the array
		*/
		inline int32 Add(ElementType&& V)
		{
			check(NumElements < MaxElements);

			Elements[NumElements] = MoveTemp(V);

			CheckSentinels();
			return NumElements++;
		}

		/**
		 * @brief Move the element to the end of the array
		*/
		inline int32 Emplace(ElementType&& V)
		{
			const int32 Index = Add(MoveTemp(V));

			CheckSentinels();

			return Index;
		}

		/**
		 * @brief Remove the element at the specified index
		 * Moves all higher elements down to fill the gap
		*/
		inline void RemoveAt(const int32 Index)
		{
			check(Index < NumElements);
			CheckSentinels();

			for (int32 MoveIndex = Index; MoveIndex < NumElements - 1; ++MoveIndex)
			{
				Elements[MoveIndex] = MoveTemp(Elements[MoveIndex + 1]);
			}
			--NumElements;

			CheckSentinels();
		}

		/**
		 * @brief Remove the element at the specified index
		 * Moves the last element into the gap.
		*/
		inline void RemoveAtSwap(const int32 Index)
		{
			check(Index < NumElements);
			CheckSentinels();

			if (Index < NumElements - 1)
			{
				Elements[Index] = MoveTemp(Elements[NumElements - 1]);
			}
			--NumElements;

			CheckSentinels();
		}

		inline ElementType* GetData()
		{
			CheckSentinels();

			return &Elements[0];
		}

		inline const ElementType* GetData() const
		{
			CheckSentinels();

			return &Elements[0];
		}

		inline ElementType* begin()
		{
			CheckSentinels();

			return &Elements[0];
		}

		inline const ElementType* begin() const
		{
			CheckSentinels();

			return &Elements[0];
		}

		inline ElementType* end()
		{
			CheckSentinels();

			return &Elements[NumElements];
		}

		inline const ElementType* end() const
		{
			CheckSentinels();

			return &Elements[NumElements];
		}

	private:
		explicit TCArray(int32 InNumElements)
		: NumElements(InNumElements)
		{
			check(Num() <= Max());
		}

		static constexpr int32 SentinelValue = 0xA1B2C3D4;

		void CheckSentinels() const
		{
#if CHAOS_CARRAY_SENTINEL
			if (!ensureAlways(Sentinel0 == SentinelValue) || !ensureAlways(Sentinel1 == SentinelValue) || !ensureAlways(Sentinel2 == SentinelValue))
			{
				UE_LOGF(LogChaos, Fatal, "TCArray Sentinel(s) Corrupted: 0x%08x, 0x%08x, 0x%08x [Expected 0x%08x]", Sentinel0, Sentinel1, Sentinel2, SentinelValue);
			}
#endif
		}

		IF_CHAOS_CARRAY_SENTINEL(int32 Sentinel0 = SentinelValue);

		// NOTE: Element count before array for better cache behaviour
		int32 NumElements;

		IF_CHAOS_CARRAY_SENTINEL(int32 Sentinel1 = SentinelValue);

		ElementType Elements[MaxElements];

		IF_CHAOS_CARRAY_SENTINEL(int32 Sentinel2 = SentinelValue);
	};


	/**
	 * @brief A fixed allocator without array bounds checking except in Debug builds.
	 *
	 * In non-debug builds this offers no saftey at all - it is effectively a C-style array.
	 *
	 * This is for use in critical path code where bounds checking would be costly and we want
	 * to ship a build with most asserts enabled (e.g. the server)
	*/
	template<int32 NumInlineElements>
	class TUncheckedFixedAllocator : public TFixedAllocator<NumInlineElements>
	{
	public:
#if !CHAOS_CHECK_UNCHECKED_ARRAY
		enum { RequireRangeCheck = false };
#endif
	};

	class FUncheckedHeapAllocator : public FHeapAllocator
	{
	public:
#if !CHAOS_CHECK_UNCHECKED_ARRAY
		enum { RequireRangeCheck = false };
#endif
	};

	template<typename T, int32 N>
	using TFixedArray = TArray<T, TFixedAllocator<N>>;

	template<typename T, int32 N>
	using TUncheckedFixedArray = TArray<T, TUncheckedFixedAllocator<N>>;

	template<typename T>
	using TUncheckedArray = TArray<T, FUncheckedHeapAllocator>;
}

template<int32 NumInlineElements>
struct TAllocatorTraits<Chaos::TUncheckedFixedAllocator<NumInlineElements>> : TAllocatorTraits<TFixedAllocator<NumInlineElements>>
{
};

template<>
struct TAllocatorTraits<Chaos::FUncheckedHeapAllocator> : TAllocatorTraits<FHeapAllocator>
{
};

template <typename T, int N>
struct TIsContiguousContainer<Chaos::TCArray<T, N>>
{
	enum { Value = true };
};

