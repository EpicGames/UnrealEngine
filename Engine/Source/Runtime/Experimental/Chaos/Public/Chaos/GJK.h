// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Box.h"
#include "Chaos/Capsule.h"
#include "Chaos/EPA.h"
#include "Chaos/GJKShape.h"
#include "Chaos/ImplicitObjectScaled.h"
#include "Chaos/Simplex.h"
#include "Chaos/Sphere.h"
#include "Math/VectorRegister.h"
#include "Chaos/VectorUtility.h"
#include "Chaos/SimplexVectorized.h"
#include "Chaos/EPAVectorized.h"

#include "ChaosCheck.h"
#include "ChaosLog.h"

// Enable some logging or ensures to trap issues in collision detection, usually related to degeneracies in GJK/EPA
#define CHAOS_COLLISIONERROR_LOG_ENABLED		((!UE_BUILD_TEST && !UE_BUILD_SHIPPING) && 0)
#define CHAOS_COLLISIONERROR_ENSURE_ENABLED		((!UE_BUILD_TEST && !UE_BUILD_SHIPPING) && 1)

#if CHAOS_COLLISIONERROR_LOG_ENABLED
#define CHAOS_COLLISIONERROR_CLOG(Condition, Fmt, ...) UE_CLOG((Condition), LogChaos, Error, Fmt, __VA_ARGS__)
#else
#define CHAOS_COLLISIONERROR_CLOG(Condition, Fmt, ...)
#endif

#if CHAOS_COLLISIONERROR_ENSURE_ENABLED
#define CHAOS_COLLISIONERROR_ENSURE(X) ensure(X)
#else
#define CHAOS_COLLISIONERROR_ENSURE(X)
#endif

namespace Chaos
{
	/*
	* To avoid code bloat from ~2k instances of templated GJKRaycast2ImplSimd, it was rewritten to operate on support function pointers and void* object pointers
	* This saved around 4MB in .text section and provided a noticeable CPU performance increase on a bottom line hardware
	*/
	struct FGeomGJKHelperSIMD
	{
		typedef VectorRegister4Float(*SupportFunc)(const void* Geom, FRealSingle Margin, const VectorRegister4Float V);

		const void* Geometry;
		mutable FRealSingle Margin;
		FRealSingle Radius;
		SupportFunc Func;

		template<class T>
		FGeomGJKHelperSIMD(const T& Geom)
			: Geometry(&Geom), Margin(Geom.GetMarginf()), Radius(Geom.GetRadiusf()), Func(&SupportCoreSimd<T>)
		{}

		FRealSingle GetRadius() const { return Radius; }
		FRealSingle GetMargin() const { return Margin; }
		FRealSingle GetRadiusf() const { return Radius; }
		FRealSingle GetMarginf() const { return Margin; }

		VectorRegister4Float operator()(const VectorRegister4Float V) const { return SupportFunction(V); }
		VectorRegister4Float SupportFunction(const VectorRegister4Float V) const { return Func(Geometry, Margin, V); }
		VectorRegister4Float SupportFunction(const VectorRegister4Float V, FRealSingle InMargin) const { return Func(Geometry, InMargin, V); }

		VectorRegister4Float operator()(const VectorRegister4Float AToBRotation, const VectorRegister4Float BToARotation, const VectorRegister4Float V) const { return SupportFunction(AToBRotation, BToARotation, V); }
		VectorRegister4Float SupportFunction(const VectorRegister4Float AToBRotation, const VectorRegister4Float BToARotation, const VectorRegister4Float V) const {
			const VectorRegister4Float VInB = VectorQuaternionRotateVector(AToBRotation, V);
			const VectorRegister4Float SupportBLocal = Func(Geometry, Margin, VInB);
			return VectorQuaternionRotateVector(BToARotation, SupportBLocal);
		}

	private:
		template<class T>
		static VectorRegister4Float SupportCoreSimd(const void* Geom, FRealSingle InMargin, const VectorRegister4Float V)
		{
			return ((const T*)Geom)->SupportCoreSimd(V, InMargin);
		}
	};

	enum EGJKTestSpace
	{
		SameSpace,
		ALocalSpace,
	};

	template <typename T, EGJKTestSpace TestSpace>
	struct TGJKSupportHelperSIMD
	{
		static void Support(const FGeomGJKHelperSIMD& RESTRICT A, const FGeomGJKHelperSIMD& RESTRICT B, const T& SearchDir, const T& AToBRotation, const T& BToARotation, T& OutSupportA, T& OutSupportB)
		{
			OutSupportA = A.SupportFunction(TMakeVectorRegisterFloatFromDouble<T>(SearchDir));
			OutSupportB = B.SupportFunction(TMakeVectorRegisterFloatFromDouble<T>(AToBRotation), TMakeVectorRegisterFloatFromDouble<T>(BToARotation), TMakeVectorRegisterFloatFromDouble<T>(VectorNegate(SearchDir)));
		}
	};

	template <typename T>
	struct TGJKSupportHelperSIMD<T, EGJKTestSpace::SameSpace>
	{
		static void Support(const FGeomGJKHelperSIMD& RESTRICT A, const FGeomGJKHelperSIMD& RESTRICT B, const T& SearchDir, const T& BToARotation, const T& AToBRotation, T& OutSupportA, T& OutSupportB)
		{
			OutSupportA = A.SupportFunction(TMakeVectorRegisterFloatFromDouble<T>(SearchDir));
			OutSupportB = B.SupportFunction(TMakeVectorRegisterFloatFromDouble<T>(VectorNegate(SearchDir)));
		}
	};

	template <EGJKTestSpace TestSpace>
	struct TSupportBAtOriginHelperSIMD
	{
		const FGeomGJKHelperSIMD& B;
		const VectorRegister4Float AToBRotation;
		const VectorRegister4Float BToARotation;
		const VectorRegister4Float StartPoint;

		TSupportBAtOriginHelperSIMD(const FGeomGJKHelperSIMD& InB, const VectorRegister4Float& InAToBRotation, const VectorRegister4Float InBToARotation, const VectorRegister4Float& InStartPoint)
			: B(InB)
			, AToBRotation(InAToBRotation)
			, BToARotation(InBToARotation)
			, StartPoint(InStartPoint)
		{
		}

		VectorRegister4Float operator()(const VectorRegister4Float& V) const { return VectorAdd(B.SupportFunction(AToBRotation, BToARotation, V), StartPoint); }
	};

	template <>
	struct TSupportBAtOriginHelperSIMD<EGJKTestSpace::SameSpace>
	{
		const FGeomGJKHelperSIMD& B;
		const VectorRegister4Float StartPoint;
		TSupportBAtOriginHelperSIMD(const FGeomGJKHelperSIMD& InB, const VectorRegister4Float& InAToBRotation, const VectorRegister4Float InBToARotation, const VectorRegister4Float& InStartPoint)
			: B(InB)
			, StartPoint(InStartPoint)
		{
		}
		VectorRegister4Float operator()(const VectorRegister4Float& V) const { return VectorAdd(B.SupportFunction(V), StartPoint); }
	};

	// Check the GJK iteration count against a limit to prevent infinite loops. We should never really hit the limit but 
	// we are seeing it happen in some cases and more often on some platforms than others so for now we have a warning when it hits.
	// @todo(chaos): track this issue down (see UE-156361)
	template<typename ConvexTypeA, typename ConvexTypeB>
	inline bool CheckGJKIterationLimit(const int32 NumIterations, const ConvexTypeA& A, const ConvexTypeB& B)
	{
		const int32 MaxIterations = 32;
		const bool bLimitExceeded = (NumIterations >= MaxIterations);

		CHAOS_COLLISIONERROR_CLOG(bLimitExceeded, TEXT("GJK hit iteration limit with shapes:\n    A: %s\n    B: %s"), *A.ToString(), *B.ToString());
		CHAOS_COLLISIONERROR_ENSURE(!bLimitExceeded);

		return bLimitExceeded;
	}


	// Calculate the margins used for queries based on shape radius, shape margins and shape types
	template <typename TGeometryA, typename TGeometryB, typename T>
	void CalculateQueryMargins(const TGeometryA& A, const TGeometryB& B, T& outMarginA, T& outMarginB)
	{
		// Margin selection logic: we only need a small margin for sweeps since we only move the sweeping object
		// to the point where it just touches.
		// Spheres and Capsules: always use the core shape and full "margin" because it represents the radius
		// Sphere/Capsule versus OtherShape: no margin on other
		// OtherShape versus OtherShape: use margin of the smaller shape, zero margin on the other
		const T RadiusA = A.GetRadiusf();
		const T RadiusB = B.GetRadiusf();
		const bool bHasRadiusA = RadiusA > 0;
		const bool bHasRadiusB = RadiusB > 0;

		// The sweep margins if required. Only one can be non-zero (we keep the smaller one)
		const T SweepMarginScale = 0.05f;
		const bool bAIsSmallest = A.GetMarginf() < B.GetMarginf();
		const T SweepMarginA = (bHasRadiusA || bHasRadiusB) ? 0.0f : (bAIsSmallest ? SweepMarginScale * A.GetMarginf() : 0.0f);
		const T SweepMarginB = (bHasRadiusA || bHasRadiusB) ? 0.0f : (bAIsSmallest ? 0.0f : SweepMarginScale * B.GetMarginf());

		// Net margin (note: both SweepMargins are zero if either Radius is non-zero, and only one SweepMargin can be non-zero)
		outMarginA = RadiusA + SweepMarginA;
		outMarginB = RadiusB + SweepMarginB;
	}
	
	/** 
		Determines if two convex geometries overlap.
		
		@param A The first geometry
		@param B The second geometry
		@param BToATM The transform of B in A's local space
		@param ThicknessA The amount of geometry inflation for Geometry A(for example if the surface distance of two geometries with thickness 0 would be 2, a thickness of 0.5 would give a distance of 1.5)
		@param InitialDir The first direction we use to search the CSO
		@return True if the geometries overlap, False otherwise 
	 */
	template <typename T, typename TGeometryA, typename TGeometryB>
	bool GJKIntersection(const TGeometryA& RESTRICT A, const TGeometryB& RESTRICT B, const TRigidTransform<T, 3>& BToATM, const T InThicknessA = 0, const TVector<T, 3>& InitialDir = TVector<T, 3>(1, 0, 0))
	{
		const T EpsilonScale = FMath::Max<T>(A.TGeometryA::BoundingBox().Extents().Max(), B.TGeometryB::BoundingBox().Extents().Max());

		const bool bUseDouble = EpsilonScale > 1.e5;
		if (bUseDouble)
		{
			return GJKIntersectionImpl<T, FReal>(A, B, EpsilonScale, BToATM, InThicknessA, InitialDir);
		}
		return GJKIntersectionImpl<T, FRealSingle>(A, B, EpsilonScale, BToATM, InThicknessA, InitialDir);
	}

	template <typename TGeometryA, typename TGeometryB>
	bool GJKIntersectionSimd(const TGeometryA& RESTRICT A, const TGeometryB& RESTRICT B, const VectorRegister4Float& Translation, const VectorRegister4Float& Rotation, FRealSingle InThicknessA, const VectorRegister4Float& InitialDir)
	{
		const FReal EpsilonScale = FMath::Max<FReal>(A.TGeometryA::BoundingBox().Extents().Max(), B.TGeometryB::BoundingBox().Extents().Max());
		return GJKIntersectionSimd(A, B, FRealSingle(EpsilonScale), Translation, Rotation, InThicknessA, InitialDir);
	}

	template <typename T>
	inline bool GJKIntersectionSimd(const FGeomGJKHelperSIMD& RESTRICT A, const FGeomGJKHelperSIMD& RESTRICT B, FRealSingle EpsilonScale, const T& Translation, const T& Rotation, FRealSingle InThicknessA, const T& InitialDir)
	{
		T V = VectorNegate(InitialDir);
		V = VectorNormalizeSafe(V, MakeVectorRegisterFloatConstant(-1.f, 0.f, 0.f, 0.f));

		const T AToBRotation = VectorQuaternionInverse(Rotation);
		bool bTerminate;
		bool bNearZero = false;
		int NumIterations = 0;
		T PrevDist2Simd = MakeVectorRegisterFloatConstant(FLT_MAX, FLT_MAX, FLT_MAX, FLT_MAX);

		T SimplexSimd[4] = { VectorZero(), VectorZero(), VectorZero(), VectorZero() };
		T BarycentricSimd;
		int32 NumVerts = 0;

		FRealSingle ThicknessA;
		FRealSingle ThicknessB;
		CalculateQueryMargins(A, B, ThicknessA, ThicknessB);
		ThicknessA += InThicknessA;

		EpsilonScale = FMath::Min<FRealSingle>(EpsilonScale, 1e5f);
		
		const FRealSingle InflationReal = ThicknessA + ThicknessB + 1e-6f * EpsilonScale;
		const T Inflation = VectorSet1(InflationReal);
		const T Inflation2 = VectorMultiply(Inflation, Inflation);

		int32 VertexIndexA = INDEX_NONE, VertexIndexB = INDEX_NONE;
		do
		{
 			if (!ensure(NumIterations++ < 32))	//todo: take this out
			{
				break;	//if taking too long just stop. This should never happen
			}
			const T NegVSimd = VectorNegate(V);
			const T SupportASimd = A.SupportFunction(TMakeVectorRegisterFloatFromDouble<T>(NegVSimd), ThicknessA);
			const T VInBSimd = VectorQuaternionRotateVector(AToBRotation, V);
			const T SupportBLocalSimd = B.SupportFunction(TMakeVectorRegisterFloatFromDouble<T>(VInBSimd), ThicknessB);
			const T SupportBSimd = VectorAdd(VectorQuaternionRotateVector(Rotation, SupportBLocalSimd), Translation);
			const T WSimd = VectorSubtract(SupportASimd, SupportBSimd);

			if (::VectorMaskBits(VectorCompareGT(VectorDot3(V, WSimd), Inflation)))
			{
				return false;
			}

			SimplexSimd[NumVerts++] = WSimd;

			V = VectorSimplexFindClosestToOrigin<T, false>(SimplexSimd, NumVerts, BarycentricSimd, nullptr, nullptr);

			const T NewDist2Simd = VectorDot3(V, V);
			bNearZero = ::VectorMaskBits(VectorCompareLT(NewDist2Simd, Inflation2)) != 0;

			//as simplices become degenerate we will stop making progress. This is a side-effect of precision, in that case take V as the current best approximation
			//question: should we take previous v in case it's better?
			bool bMadeProgress = ::VectorMaskBits(VectorCompareLT(NewDist2Simd, PrevDist2Simd)) != 0;
			bTerminate = bNearZero || !bMadeProgress;
			PrevDist2Simd = NewDist2Simd;

			if (!bTerminate)
			{
				V = VectorDivide(V, VectorSqrt(NewDist2Simd));
			}

		} while (!bTerminate);

		return bNearZero;
	}

	template <typename PrevT, typename NewT>
	bool GJKIntersectionImpl(const FGeomGJKHelperSIMD& RESTRICT A, const FGeomGJKHelperSIMD& RESTRICT B, PrevT EpsilonScale, const TRigidTransform<PrevT, 3>& BToATM, const PrevT InThicknessA = 0, const TVector<PrevT, 3>& InitialDir = TVector<PrevT, 3>(1, 0, 0))
	{
		const UE::Math::TQuat<PrevT>& Rotation = BToATM.GetRotation();
		TVectorRegisterType<NewT> RotationSimd = MakeVectorRegister(NewT(Rotation.X), NewT(Rotation.Y), NewT(Rotation.Z), NewT(Rotation.W));

		const UE::Math::TVector<PrevT>& TranslationDouble = BToATM.GetTranslation();
		const TVectorRegisterType<NewT> TranslationSimd = MakeVectorRegister(NewT(TranslationDouble.X), NewT(TranslationDouble.Y), NewT(TranslationDouble.Z), NewT(0.0));
		RotationSimd = VectorNormalizeSafe(RotationSimd, GlobalVectorConstants::Float0001);

		const TVectorRegisterType<NewT> InitialDirSimd = MakeVectorRegister(NewT(InitialDir[0]), NewT(InitialDir[1]), NewT(InitialDir[2]), NewT(0.0));

		return GJKIntersectionSimd<TVectorRegisterType<NewT>>(A, B, FRealSingle(EpsilonScale), TranslationSimd, RotationSimd, FRealSingle(InThicknessA), InitialDirSimd);
	}

	/** 
		Determines if two convex geometries in the same space overlap
		IMPORTANT: the two convex geometries must be in the same space!

		@param A The first geometry
		@param B The second geometry
		@param ThicknessA The amount of geometry inflation for Geometry A(for example if the surface distance of two geometries with thickness 0 would be 2, a thickness of 0.5 would give a distance of 1.5)
		@param InitialDir The first direction we use to search the CSO (Must be normalized)
		@param ThicknessB The amount of geometry inflation for Geometry B(for example if the surface distance of two geometries with thickness 0 would be 2, a thickness of 0.5 would give a distance of 1.5)
		@return True if the geometries overlap, False otherwise 
	*/

	template <typename TGeometryA, typename TGeometryB>
	bool GJKIntersectionSameSpaceSimd(const TGeometryA& A, const TGeometryB& B, FRealSingle InThicknessA, const VectorRegister4Float& InitialDir)
	{
		VectorRegister4Float V = VectorNegate(InitialDir);

		//ensure(FMath::Abs(VectorDot3Scalar(V, V) - 1.0f) < 1e-3f ); // Check that the vector is normalized, comment out for performance

		FSimplex SimplexIDs;
		VectorRegister4Float Simplex[4] = { VectorZero(), VectorZero(), VectorZero(), VectorZero() };
		int32 NumVerts = 0;
		VectorRegister4Float Barycentric = VectorZero();

		bool bTerminate;
		bool bNearZero = false;
		int NumIterations = 0;
		VectorRegister4Float PrevDist2 = GlobalVectorConstants::BigNumber;
		FRealSingle ThicknessA;
		FRealSingle ThicknessB;
		CalculateQueryMargins(A, B, ThicknessA, ThicknessB);
		ThicknessA += InThicknessA;

		const FRealSingle Inflation = ThicknessA + ThicknessB + 1e-3f;
		const FRealSingle Inflation2 = Inflation * Inflation;

		VectorRegister4Float InflationSimd = VectorSet1(Inflation);
		VectorRegister4Float Inflation2Simd = VectorMultiply(InflationSimd, InflationSimd);
		do
		{
			if (!ensure(NumIterations++ < 32))	//todo: take this out
			{
				break;	//if taking too long just stop. This should never happen
			}
			const VectorRegister4Float NegV = VectorNegate(V);
			const VectorRegister4Float SupportA = A.SupportCoreSimd(NegV, ThicknessA);
			const VectorRegister4Float VInB = V; // same space
			const VectorRegister4Float SupportB = B.SupportCoreSimd(VInB, ThicknessB);
			const VectorRegister4Float W = VectorSubtract(SupportA, SupportB);

			VectorRegister4Float VDotW = VectorDot3(V, W);

			if (::VectorMaskBits(VectorCompareGT(VDotW, InflationSimd)))
			{
				return false;
			}

			Simplex[NumVerts++] = W;

			V = VectorSimplexFindClosestToOrigin<VectorRegister4Float, false>(Simplex, NumVerts, Barycentric, nullptr, nullptr);
			const VectorRegister4Float NewDist2 = VectorDot3(V, V);
			
			bNearZero = static_cast<bool>(::VectorMaskBits(VectorCompareGT(Inflation2Simd, NewDist2)));

			//as simplices become degenerate we will stop making progress. This is a side-effect of precision, in that case take V as the current best approximation
			//question: should we take previous v in case it's better?
			const bool bMadeProgress = static_cast<bool>(::VectorMaskBits(VectorCompareGT(PrevDist2, NewDist2)));
			bTerminate = bNearZero || !bMadeProgress;

			PrevDist2 = NewDist2;

			if (!bTerminate)
			{
				V = VectorDivide(V, VectorSqrt(NewDist2));
			}

		} while (!bTerminate);

		return bNearZero;
	}


	/**
	 * @brief Internal simplex data for GJK that can also be stored for warm-starting subsequent calls
	 * @tparam T the number type (float or double)
	 *
	 * @see GJKPenetrationWarmStartable
	*/
	template<typename T>
	class TGJKSimplexData
	{
	public:
		TGJKSimplexData()
			: NumVerts(0)
		{}

		// Clear the data - used to start a GJK search from the default search direction
		void Reset()
		{
			NumVerts = 0;
		}

		// Save any data that was not directly updated while iterating in GJK
		void Save(const FSimplex InSimplexIDs)
		{
			// We don't need to store the simplex vertex order because the indices are always
			// sorted at the end of each iteration. We just need to know how many vertices we have.
			NumVerts = InSimplexIDs.NumVerts;
		}

		// Recompute the Simplex and separating vector from the stored data at the current relative transform
		// This aborts if we have no simplex data to restore or the origin is inside the simplex. Outputs must already 
		// have reasonable default values for running GJK without a warm-start.
		void Restore(const TRigidTransform<T, 3>& BToATM, FSimplex& OutSimplexIDs, TVec3<T> OutSimplex[], TVec3<T>& OutV, T& OutDistance, const T Epsilon)
		{
			if (NumVerts > 0)
			{
				OutSimplexIDs.NumVerts = NumVerts;

				for (int32 VertIndex = 0; VertIndex < NumVerts; ++VertIndex)
				{
					OutSimplexIDs.Idxs[VertIndex] = VertIndex;
					OutSimplex[VertIndex] = As[VertIndex] - BToATM.TransformPositionNoScale(Bs[VertIndex]);
				}

				const TVec3<T> V = SimplexFindClosestToOrigin(OutSimplex, OutSimplexIDs, Barycentric, As, Bs);
				const T Distance = V.Size();

				// If the origin is inside the simplex at the new transform, we need to abort the restore
				// This is necessary to cover the very-small separation case where we use the normal
				// calculated in the previous iteration in GJKL, but we have no way to restore that.
				// Note: we have already written to the simplex but that's ok because we reset the vert count.
				if (Distance > Epsilon)
				{
					OutV = V / Distance;
					OutDistance = Distance;
				}
				else
				{
					OutSimplexIDs.NumVerts = 0;
				}
			}
		}

		void Restore2(const TRigidTransform<T, 3>& BToATM, int32& OutNumVerts, TVec3<T> OutSimplex[], TVec3<T>& OutV, T& OutDistance, const T Epsilon)
		{
			OutNumVerts = 0;

			if (NumVerts > 0)
			{
				for (int32 VertIndex = 0; VertIndex < NumVerts; ++VertIndex)
				{
					OutSimplex[VertIndex] = As[VertIndex] - BToATM.TransformPositionNoScale(Bs[VertIndex]);
				}

				const TVec3<T> V = SimplexFindClosestToOrigin2(OutSimplex, NumVerts, Barycentric, As, Bs);
				const T DistanceSq = V.SizeSquared();

				// If the origin is inside the simplex at the new transform, we need to abort the restore
				// This is necessary to cover the very-small separation case where we use the normal
				// calculated in the previous iteration in GJKL, but we have no way to restore that.
				// Note: we have already written to the simplex but that's ok because we reset the vert count.
				if (DistanceSq > FMath::Square(Epsilon))
				{
					const T Distance = FMath::Sqrt(DistanceSq);
					OutNumVerts = NumVerts;
					OutV = V / Distance;
					OutDistance = Distance;
				}
			}
		}


		// Maximum number of vertices that a GJK simplex can have
		static const int32 MaxSimplexVerts = 4;

		// Simplex vertices on shape A, in A-local space
		TVec3<T> As[MaxSimplexVerts];

		// Simplex vertices on shape B, in B-local space
		TVec3<T> Bs[MaxSimplexVerts];

		// Barycentric coordinates of closest point to origin on the simplex
		T Barycentric[MaxSimplexVerts];

		// Number of vertices in the simplex. Up to 4.
		int32 NumVerts;
	};

	// GJK warm-start data at default numeric precision
	using FGJKSimplexData = TGJKSimplexData<FReal>;

	struct FGeomGJKHelper
	{
		typedef FVector(*SupportFunc)(const void* Geom, const FVec3& Direction, FReal* OutSupportDelta, int32& VertexIndex);

		const void* Geometry;
		SupportFunc Func;
		FRealSingle Margin;

		template<class T>
		FGeomGJKHelper(const T& Geom) : Geometry(&Geom), Func(&SupportCore<T>), Margin(Geom.GetMarginf()) {}

		FVector SupportFunction(const FVec3& V, int32& VertexIndex) const { return Func(Geometry, V, nullptr, VertexIndex); }
		FVector SupportFunction(const FVec3& V, FReal* OutSupportDelta, int32& VertexIndex) const { return Func(Geometry, V, OutSupportDelta, VertexIndex); }

		FVector SupportFunction(const FRotation3& AToBRotation, const FVec3& V, FReal* OutSupportDelta, int32& VertexIndex) const
		{
			const FVec3 VInB = AToBRotation * V;
			return Func(Geometry, VInB, OutSupportDelta, VertexIndex);
		}

		FVector SupportFunction(const FRigidTransform3& BToATM, const FRotation3& AToBRotation, const FVec3& V, int32& VertexIndex) const
		{
			const FVector VInB = AToBRotation * V;
			const FVector SupportBLocal = Func(Geometry, VInB, nullptr, VertexIndex);
			return BToATM.TransformPositionNoScale(SupportBLocal);
		}

		FVector SupportFunction(const FRigidTransform3& BToATM, const FRotation3& AToBRotation, const FVec3& V, FReal* OutSupportDelta, int32& VertexIndex) const
		{
			const FVector VInB = AToBRotation * V;
			const FVector SupportBLocal = Func(Geometry, VInB, OutSupportDelta, VertexIndex);
			return BToATM.TransformPositionNoScale(SupportBLocal);
		}

		FReal GetMargin() const { return Margin; }
		FReal GetMarginf() const { return Margin; }

	private:
		template<class T>
		static FVector SupportCore(const void* Geom, const FVec3& Direction, FReal* OutSupportDelta, int32& VertexIndex)
		{
			const T* Geometry = (const T*)Geom;
			return Geometry->SupportCore(Direction, Geometry->GetMarginf(), OutSupportDelta, VertexIndex);
		}
	};

	/**
	 * @brief Calculate the penetration data for two shapes using GJK and a warm-start buffer.
	 *
	 * @tparam T Number type (float or double)
	 * @tparam TGeometryA The first shape type
	 * @tparam TGeometryB The second shape type
	 * @param A The first shape
	 * @param B The second shape
	 * @param BToATM A transform from B-local space to A-local space
	 * @param OutPenetration The overlap distance (+ve for overlap, -ve for separation)
	 * @param OutClosestA The closest point on A, in A-local space
	 * @param OutClosestB The closest point on B, in B-local space
	 * @param OutNormalA The contact normal pointing away from A in A-local space
	 * @param OutNormalB The contact normal pointing away from A in B-local space
	 * @param OutVertexA The closest vertex on A
	 * @param OutVertexB The closest vertex on B
	 * @param SimplexData In/out simplex data used to initialize and update GJK. Can be stored to improve convergence on subsequent calls for "small" changes in relative rotation.
	 * @param OutMaxContactDelta The maximum error in the contact position as a result of using a margin. This is the difference between the core shape + margin and the outer shape on the supporting vertices
	 * @param Epsilon The separation distance below which GJK aborts and switches to EPA
	 * @return true if the results are populated (always for this implementation, but @see EPA())
	 *
	 * The WarmStartData is an input and output parameter. If the function is called with
	 * a small change in BToATM there's it will converge much faster, usually in 1 iteration
	 * for polygonal shapes.
	 * 
	 * @note This version returns OutClosestB in B's local space, compated to GJKPenetration where all output is in the space of A
	 * 
	 * @todo(chaos): convert GJKPenetration() to use this version (but see note above)
	 *
	*/
	template <typename T, typename TGeometryA, typename TGeometryB>
	bool GJKPenetrationWarmStartable(const TGeometryA& A, const TGeometryB& B, const TRigidTransform<T, 3>& BToATM, T& OutPenetration, TVec3<T>& OutClosestA, TVec3<T>& OutClosestB, TVec3<T>& OutNormalA, TVec3<T>& OutNormalB, int32& OutVertexA, int32& OutVertexB, TGJKSimplexData<T>& InOutSimplexData, T& OutMaxSupportDelta, const T Epsilon = T(1.e-3), const T EPAEpsilon = T(1.e-2))
	{
		T SupportDeltaA = 0;
		T SupportDeltaB = 0;
		T MaxSupportDelta = 0;

		int32& VertexIndexA = OutVertexA;
		int32& VertexIndexB = OutVertexB;

		// Return the support vertex in A-local space for a vector V in A-local space
		auto SupportAFunc = [&A, &SupportDeltaA, &VertexIndexA](const TVec3<T>& V)
		{
			return A.SupportCore(V, A.GetMarginf(), &SupportDeltaA, VertexIndexA);
		};

		const TRotation<T, 3> AToBRotation = BToATM.GetRotation().Inverse();

		// Return the support point on B, in B-local space, for a vector V in A-local space
		auto SupportBFunc = [&B, &AToBRotation, &SupportDeltaB, &VertexIndexB](const TVec3<T>& V)
		{
			const TVec3<T> VInB = AToBRotation * V;
			return B.SupportCore(VInB, B.GetMarginf(), &SupportDeltaB, VertexIndexB);
		};

		// V and Simplex are in A-local space
		TVec3<T> V = TVec3<T>(-1, 0, 0);
		TVec3<T> Simplex[4];
		FSimplex SimplexIDs;
		T Distance = FLT_MAX;

		// If we have warm-start data, rebuild the simplex from the stored data
		InOutSimplexData.Restore(BToATM, SimplexIDs, Simplex, V, Distance, Epsilon);

		TVec3<T> Normal = -V;					// Remember the last good normal (i.e. don't update it if separation goes less than Epsilon and we can no longer normalize)
		bool bIsDegenerate = false;				// True if GJK cannot make any more progress
		bool bIsContact = false;				// True if shapes are within Epsilon or overlapping - GJK cannot provide a solution
		int32 NumIterations = 0;
		const T ThicknessA = A.GetMarginf();
		const T ThicknessB = B.GetMarginf();
		const T SeparatedDistance = ThicknessA + ThicknessB + Epsilon;
		while (!bIsContact && !bIsDegenerate)
		{
			// If taking too long just stop
			++NumIterations;
			if (CheckGJKIterationLimit(NumIterations, A, B))
			{
				break;	
			}

			const TVec3<T> NegV = -V;
			const TVec3<T> SupportA = SupportAFunc(NegV);
			const TVec3<T> SupportB = SupportBFunc(V);
			const TVec3<T> SupportBInA = BToATM.TransformPositionNoScale(SupportB);
			const TVec3<T> W = SupportA - SupportBInA;
			MaxSupportDelta = FMath::Max(SupportDeltaA, SupportDeltaB);

			// If we didn't move to at least ConvergedDistance or closer, assume we have reached a minimum
			const T ConvergenceTolerance = 1.e-4f;
			const T ConvergedDistance = (1.0f - ConvergenceTolerance) * Distance;
			const T VW = TVec3<T>::DotProduct(V, W);

			InOutSimplexData.As[SimplexIDs.NumVerts] = SupportA;
			InOutSimplexData.Bs[SimplexIDs.NumVerts] = SupportB;
			SimplexIDs[SimplexIDs.NumVerts] = SimplexIDs.NumVerts;
			Simplex[SimplexIDs.NumVerts++] = W;

			V = SimplexFindClosestToOrigin(Simplex, SimplexIDs, InOutSimplexData.Barycentric, InOutSimplexData.As, InOutSimplexData.Bs);
			T NewDistance = V.Size();

			// Are we overlapping or too close for GJK to get a good result?
			bIsContact = (NewDistance < Epsilon);

			// If we did not get closer in this iteration, we are in a degenerate situation
			bIsDegenerate = (NewDistance >= Distance);

			// If we are not too close, update the separating vector and keep iterating
			// If we are too close we drop through to EPA, but if EPA determines the shapes are actually separated 
			// after all, we will wend up using the normal from the previous loop
			if (!bIsContact)
			{
				V /= NewDistance;
				Normal = -V;
			}
			Distance = NewDistance;
		}
		// Save the warm-start data (not much to store since we updated most of it in the loop above)
		InOutSimplexData.Save(SimplexIDs);
		if (bIsContact)
		{
			// We did not converge or we detected an overlap situation, so run EPA to get contact data
			// Rebuild the simplex into a variable sized array - EPA can generate any number of vertices
			TArray<TVec3<T>> VertsA;
			TArray<TVec3<T>> VertsB;
			VertsA.Reserve(8);
			VertsB.Reserve(8);
			for (int i = 0; i < SimplexIDs.NumVerts; ++i)
			{
				VertsA.Add(InOutSimplexData.As[i]);
				VertsB.Add(BToATM.TransformPositionNoScale(InOutSimplexData.Bs[i]));
			}
			
			auto SupportBInAFunc = [&B, &BToATM, &AToBRotation, &SupportDeltaB, &VertexIndexB](const TVec3<T>& V)
			{
				const TVec3<T> VInB = AToBRotation * V;
				const TVec3<T> SupportBLocal = B.SupportCore(VInB, B.GetMarginf(), &SupportDeltaB, VertexIndexB);
				return BToATM.TransformPositionNoScale(SupportBLocal);
			};

			T Penetration;
			TVec3<T> MTD, ClosestA, ClosestBInA;
			const EEPAResult EPAResult = EPA<T>(VertsA, VertsB, SupportAFunc, SupportBInAFunc, Penetration, MTD, ClosestA, ClosestBInA, EPAEpsilon);

			switch (EPAResult)
			{
			case EEPAResult::MaxIterations:
				// Possibly a solution but with unknown error. Just return the last EPA state. 
				// Fall through...
			case EEPAResult::Ok:
				// EPA has a solution - return it now
				OutNormalA = MTD;
				OutNormalB = BToATM.InverseTransformVectorNoScale(MTD);
				OutPenetration = Penetration + ThicknessA + ThicknessB;
				OutClosestA = ClosestA + MTD * ThicknessA;
				OutClosestB = BToATM.InverseTransformPositionNoScale(ClosestBInA - MTD * ThicknessB);
				OutMaxSupportDelta = MaxSupportDelta;
				return true;
			case EEPAResult::BadInitialSimplex:
				// The origin is outside the simplex. Must be a touching contact and EPA setup will have calculated the normal
				// and penetration but we keep the position generated by GJK.
				Normal = MTD;
				Distance = -Penetration;
				//UE_LOGF(LogChaos, Warning, "EPA Touching Case");
				break;
			case EEPAResult::Degenerate:
				// We hit a degenerate simplex condition and could not reach a solution so use whatever near touching point GJK came up with.
				// The result from EPA under these circumstances is not usable.
				//UE_LOGF(LogChaos, Warning, "EPA Degenerate Case");
				break;
			}
		}

		// @todo(chaos): handle the case where EPA hits a degenerate triangle in the simplex.
		// Currently we return a touching contact with the last position and normal from GJK. We could run SAT instead.

		// GJK converged or we have a touching contact
		{
			TVec3<T> ClosestA(0);
			TVec3<T> ClosestB(0);
			for (int i = 0; i < SimplexIDs.NumVerts; ++i)
			{
				ClosestA += InOutSimplexData.As[i] * InOutSimplexData.Barycentric[i];
				ClosestB += InOutSimplexData.Bs[i] * InOutSimplexData.Barycentric[i];
			}

			OutNormalA = Normal;
			OutNormalB = BToATM.InverseTransformVectorNoScale(Normal);

			T Penetration = ThicknessA + ThicknessB - Distance;
			OutPenetration = Penetration;
			OutClosestA = ClosestA + OutNormalA * ThicknessA;
			OutClosestB = ClosestB - OutNormalB * ThicknessB;

			OutMaxSupportDelta = MaxSupportDelta;

			// @todo(chaos): we should pass back failure for the degenerate case so we can decide how to handle it externally.
			return true;
		}
	}

	// Same as GJKPenetrationWarmStartable but with an index-less algorithm
	template <typename T, typename TGeometryA, typename TGeometryB>
	bool GJKPenetrationWarmStartable2(const TGeometryA& A, const TGeometryB& B, const TRigidTransform<T, 3>& BToATM, T& OutPenetration, TVec3<T>& OutClosestA, TVec3<T>& OutClosestB, TVec3<T>& OutNormalA, TVec3<T>& OutNormalB, int32& OutVertexA, int32& OutVertexB, TGJKSimplexData<T>& InOutSimplexData, T& OutMaxSupportDelta, const T Epsilon = T(1.e-3), const T EPAEpsilon = T(1.e-2))
	{
		T SupportDeltaA = 0;
		T SupportDeltaB = 0;
		T MaxSupportDelta = 0;

		int32& VertexIndexA = OutVertexA;
		int32& VertexIndexB = OutVertexB;

		// Return the support vertex in A-local space for a vector V in A-local space
		auto SupportAFunc = [&A, &SupportDeltaA, &VertexIndexA](const TVec3<T>& V)
		{
			return A.SupportCore(V, A.GetMargin(), &SupportDeltaA, VertexIndexA);
		};

		const TRotation<T, 3> AToBRotation = BToATM.GetRotation().Inverse();

		// Return the support point on B, in B-local space, for a vector V in A-local space
		auto SupportBFunc = [&B, &AToBRotation, &SupportDeltaB, &VertexIndexB](const TVec3<T>& V)
		{
			const TVec3<T> VInB = AToBRotation * V;
			return B.SupportCore(VInB, B.GetMargin(), &SupportDeltaB, VertexIndexB);
		};

		// V and Simplex are in A-local space
		TVec3<T> V = TVec3<T>(-1, 0, 0);
		TVec3<T> Simplex[4];
		int32 NumVerts = 0;
		T Distance = FLT_MAX;

		// If we have warm-start data, rebuild the simplex from the stored data
		InOutSimplexData.Restore2(BToATM, NumVerts, Simplex, V, Distance, Epsilon);

		TVec3<T> Normal = -V;					// Remember the last good normal (i.e. don't update it if separation goes less than Epsilon and we can no longer normalize)
		bool bIsResult = false;					// True if GJK cannot make any more progress
		bool bIsContact = false;				// True if shapes are within Epsilon or overlapping - GJK cannot provide a solution
		int32 NumIterations = 0;
		const T ThicknessA = A.GetMargin();
		const T ThicknessB = B.GetMargin();
		while (!bIsContact && !bIsResult)
		{
			// If taking too long just stop
			++NumIterations;
			if (CheckGJKIterationLimit(NumIterations, A, B))
			{
				break;
			}

			const TVec3<T> NegV = -V;
			const TVec3<T> SupportA = SupportAFunc(NegV);
			const TVec3<T> SupportB = SupportBFunc(V);
			const TVec3<T> SupportBInA = BToATM.TransformPositionNoScale(SupportB);
			const TVec3<T> W = SupportA - SupportBInA;
			MaxSupportDelta = FMath::Max(SupportDeltaA, SupportDeltaB);

			// If we didn't move to at least ConvergedDistance or closer, assume we have reached a minimum
			const T ConvergenceTolerance = 1.e-4f;
			const T ConvergedDistance = (1.0f - ConvergenceTolerance) * Distance;
			const T VW = TVec3<T>::DotProduct(V, W);

			InOutSimplexData.As[NumVerts] = SupportA;
			InOutSimplexData.Bs[NumVerts] = SupportB;
			Simplex[NumVerts++] = W;

			V = SimplexFindClosestToOrigin2(Simplex, NumVerts, InOutSimplexData.Barycentric, InOutSimplexData.As, InOutSimplexData.Bs);
			T NewDistance = V.Size();

			// Are we overlapping or too close for GJK to get a good result?
			bIsContact = (NewDistance < Epsilon);

			// If we did not get closer in this iteration, we are in a degenerate situation or we have the result
			bIsResult = (NewDistance >= (Distance - Epsilon));

			// If we are not too close, update the separating vector and keep iterating
			// If we are too close we drop through to EPA, but if EPA determines the shapes are actually separated 
			// after all, we will end up using the normal from the previous loop
			if (!bIsContact)
			{
				V /= NewDistance;
				Normal = -V;
			}
			Distance = NewDistance;
		}
		
		// Save the warm-start data (not much to store since we updated most of it in the loop above)
		InOutSimplexData.NumVerts = NumVerts;

		if (bIsContact)
		{
			// We did not converge or we detected an overlap situation, so run EPA to get contact data
			// Rebuild the simplex into a variable sized array - EPA can generate any number of vertices
			TArray<TVec3<T>> VertsA;
			TArray<TVec3<T>> VertsB;
			VertsA.Reserve(8);
			VertsB.Reserve(8);
			for (int i = 0; i < NumVerts; ++i)
			{
				VertsA.Add(InOutSimplexData.As[i]);
				VertsB.Add(BToATM.TransformPositionNoScale(InOutSimplexData.Bs[i]));
			}

			auto SupportBInAFunc = [&B, &BToATM, &AToBRotation, &SupportDeltaB, &VertexIndexB](const TVec3<T>& V)
			{
				const TVec3<T> VInB = AToBRotation * V;
				const TVec3<T> SupportBLocal = B.SupportCore(VInB, B.GetMargin(), &SupportDeltaB, VertexIndexB);
				return BToATM.TransformPositionNoScale(SupportBLocal);
			};

			T Penetration;
			TVec3<T> MTD, ClosestA, ClosestBInA;
			const EEPAResult EPAResult = EPA<T>(VertsA, VertsB, SupportAFunc, SupportBInAFunc, Penetration, MTD, ClosestA, ClosestBInA, EPAEpsilon);

			switch (EPAResult)
			{
			case EEPAResult::MaxIterations:
				// Possibly a solution but with unknown error. Just return the last EPA state. 
				// Fall through...
			case EEPAResult::Ok:
				// EPA has a solution - return it now
				OutNormalA = MTD;
				OutNormalB = BToATM.InverseTransformVectorNoScale(MTD);
				OutPenetration = Penetration + ThicknessA + ThicknessB;
				OutClosestA = ClosestA + MTD * ThicknessA;
				OutClosestB = BToATM.InverseTransformPositionNoScale(ClosestBInA - MTD * ThicknessB);
				OutMaxSupportDelta = MaxSupportDelta;
				return true;
			case EEPAResult::BadInitialSimplex:
				// The origin is outside the simplex. Must be a touching contact and EPA setup will have calculated the normal
				// and penetration but we keep the position generated by GJK.
				Normal = MTD;
				Distance = -Penetration;
				break;
			case EEPAResult::Degenerate:
				// We hit a degenerate simplex condition and could not reach a solution so use whatever near touching point GJK came up with.
				// The result from EPA under these circumstances is not usable.
				//UE_LOGF(LogChaos, Warning, "EPA Degenerate Case");
				break;
			}
		}

		// @todo(chaos): handle the case where EPA hits a degenerate triangle in the simplex.
		// Currently we return a touching contact with the last position and normal from GJK. We could run SAT instead.

		// GJK converged or we have a touching contact
		{
			TVec3<T> ClosestA(0);
			TVec3<T> ClosestB(0);
			for (int i = 0; i < NumVerts; ++i)
			{
				ClosestA += InOutSimplexData.As[i] * InOutSimplexData.Barycentric[i];
				ClosestB += InOutSimplexData.Bs[i] * InOutSimplexData.Barycentric[i];
			}

			OutNormalA = Normal;
			OutNormalB = BToATM.InverseTransformVectorNoScale(Normal);

			T Penetration = ThicknessA + ThicknessB - Distance;
			OutPenetration = Penetration;
			OutClosestA = ClosestA + OutNormalA * ThicknessA;
			OutClosestB = ClosestB - OutNormalB * ThicknessB;

			OutMaxSupportDelta = MaxSupportDelta;

			// @todo(chaos): we should pass back failure for the degenerate case so we can decide how to handle it externally.
			return true;
		}
	}

	/**
	 * @brief Calculate the penetration data for two shapes using GJK, assuming both shapes are already in the same space.
	 * This is intended for use with triangles which have been transformed into the space of the convex shape.
	 * 
	 * @tparam T
	 * @tparam TGeometryA First geometry type
	 * @tparam TGeometryB Second geometry type. Usually FTriangle
	 * @param A First geometry
	 * @param B Second geometry (usually triangle)
	 * @param OutPenetration penetration depth (negative for separation)
	 * @param OutClosestA Closest point on A
	 * @param OutClosestB Closest point on B
	 * @param OutNormal Contact normal (points from A to B)
	 * @param OutVertexA The closest vertex on A
	 * @param OutVertexB The closest vertex on B
	 * @param OutMaxSupportDelta When the convex has a margin, an upper bounds on the distance from the contact point to the vertex on the outer hull
	 * @param Epsilon GJK tolerance
	 * @param EPAEpsilon EPA tolerance
	 * @return true always
	*/
	template <typename T, typename TGeometryA, typename TGeometryB>
	bool GJKPenetrationSameSpace(const TGeometryA& A, const TGeometryB& B, T& OutPenetration, TVec3<T>& OutClosestA, TVec3<T>& OutClosestB, TVec3<T>& OutNormal, int32& OutVertexA, int32& OutVertexB, T& OutMaxSupportDelta, const TVec3<T>& InitialDir, const T Epsilon = T(1.e-3), const T EPAEpsilon = T(1.e-2))
	{
		TGJKSimplexData<T> SimplexData;
		T SupportDeltaA = 0;
		T SupportDeltaB = 0;
		T MaxSupportDelta = 0;

		int32& VertexIndexA = OutVertexA;
		int32& VertexIndexB = OutVertexB;

		// Return the support vertex in A-local space for a vector V in A-local space
		auto SupportAFunc = [&A, &SupportDeltaA, &VertexIndexA](const TVec3<T>& V)
		{
			return A.SupportCore(V, A.GetMargin(), &SupportDeltaA, VertexIndexA);
		};

		// Return the support point on B, in B-local space, for a vector V in A-local space
		auto SupportBFunc = [&B, &SupportDeltaB, &VertexIndexB](const TVec3<T>& V)
		{
			return B.SupportCore(V, B.GetMargin(), &SupportDeltaB, VertexIndexB);
		};

		// V and Simplex are in A-local space
		TVec3<T> V = InitialDir;
		TVec3<T> Simplex[4];
		FSimplex SimplexIDs;
		T Distance = FLT_MAX;

		TVec3<T> Normal = -V;					// Remember the last good normal (i.e. don't update it if separation goes less than Epsilon and we can no longer normalize)
		bool bIsDegenerate = false;				// True if GJK cannot make any more progress
		bool bIsContact = false;				// True if shapes are within Epsilon or overlapping - GJK cannot provide a solution
		int32 NumIterations = 0;
		const T ThicknessA = A.GetMargin();
		const T ThicknessB = B.GetMargin();
		const T SeparatedDistance = ThicknessA + ThicknessB + Epsilon;
		while (!bIsContact && !bIsDegenerate)
		{
			// If taking too long just stop
			++NumIterations;
			if (CheckGJKIterationLimit(NumIterations, A, B))
			{
				break;
			}

			const TVec3<T> NegV = -V;
			const TVec3<T> SupportA = SupportAFunc(NegV);
			const TVec3<T> SupportB = SupportBFunc(V);
			const TVec3<T> W = SupportA - SupportB;
			MaxSupportDelta = FMath::Max(SupportDeltaA, SupportDeltaB);

			// If we didn't move to at least ConvergedDistance or closer, assume we have reached a minimum
			const T ConvergenceTolerance = 1.e-4f;
			const T ConvergedDistance = (1.0f - ConvergenceTolerance) * Distance;
			const T VW = TVec3<T>::DotProduct(V, W);

			SimplexData.As[SimplexIDs.NumVerts] = SupportA;
			SimplexData.Bs[SimplexIDs.NumVerts] = SupportB;
			SimplexIDs[SimplexIDs.NumVerts] = SimplexIDs.NumVerts;
			Simplex[SimplexIDs.NumVerts++] = W;

			V = SimplexFindClosestToOrigin(Simplex, SimplexIDs, SimplexData.Barycentric, SimplexData.As, SimplexData.Bs);
			T NewDistance = V.Size();

			// Are we overlapping or too close for GJK to get a good result?
			bIsContact = (NewDistance < Epsilon);

			// If we did not get closer in this iteration, we are in a degenerate situation
			bIsDegenerate = (NewDistance >= Distance);

			// If we are not too close, update the separating vector and keep iterating
			// If we are too close we drop through to EPA, but if EPA determines the shapes are actually separated 
			// after all, we will wend up using the normal from the previous loop
			if (!bIsContact)
			{
				V /= NewDistance;
				Normal = -V;
			}
			Distance = NewDistance;
		}

		// Save the warm-start data (not much to store since we updated most of it in the loop above)
		SimplexData.Save(SimplexIDs);

		if (bIsContact)
		{
			// We did not converge or we detected an overlap situation, so run EPA to get contact data
			// Rebuild the simplex into a variable sized array - EPA can generate any mumber of vertices
			TArray<TVec3<T>> VertsA;
			TArray<TVec3<T>> VertsB;
			VertsA.Reserve(8);
			VertsB.Reserve(8);
			for (int i = 0; i < SimplexIDs.NumVerts; ++i)
			{
				VertsA.Add(SimplexData.As[i]);
				VertsB.Add(SimplexData.Bs[i]);
			}

			T Penetration;
			TVec3<T> MTD, ClosestA, ClosestB;
			const EEPAResult EPAResult = EPA<T>(VertsA, VertsB, SupportAFunc, SupportBFunc, Penetration, MTD, ClosestA, ClosestB, EPAEpsilon);

			switch (EPAResult)
			{
			case EEPAResult::MaxIterations:
				// Possibly a solution but with unknown error. Just return the last EPA state. 
				// Fall through...
			case EEPAResult::Ok:
				// EPA has a solution - return it now
				OutNormal = MTD;
				OutPenetration = Penetration + ThicknessA + ThicknessB;
				OutClosestA = ClosestA + MTD * ThicknessA;
				OutClosestB = ClosestB - MTD * ThicknessB;
				OutMaxSupportDelta = MaxSupportDelta;
				return true;
			case EEPAResult::BadInitialSimplex:
				// The origin is outside the simplex. Must be a touching contact and EPA setup will have calculated the normal
				// and penetration but we keep the position generated by GJK.
				Normal = MTD;
				Distance = -Penetration;
				//UE_LOGF(LogChaos, Warning, "EPA Touching Case");
				break;
			case EEPAResult::Degenerate:
				// We hit a degenerate simplex condition and could not reach a solution so use whatever near touching point GJK came up with.
				// The result from EPA under these circumstances is not usable.
				//UE_LOGF(LogChaos, Warning, "EPA Degenerate Case");
				break;
			}
		}

		// @todo(chaos): handle the case where EPA hits a degenerate triangle in the simplex.
		// Currently we return a touching contact with the last position and normal from GJK. We could run SAT instead.

		// GJK converged or we have a touching contact
		{
			TVec3<T> ClosestA(0);
			TVec3<T> ClosestB(0);
			for (int i = 0; i < SimplexIDs.NumVerts; ++i)
			{
				ClosestA += SimplexData.As[i] * SimplexData.Barycentric[i];
				ClosestB += SimplexData.Bs[i] * SimplexData.Barycentric[i];
			}

			OutPenetration = ThicknessA + ThicknessB - Distance;
			OutClosestA = ClosestA + Normal * ThicknessA;
			OutClosestB = ClosestB - Normal * ThicknessB;
			OutNormal = Normal;
			OutMaxSupportDelta = MaxSupportDelta;

			// @todo(chaos): we should pass back failure for the degenerate case so we can decide how to handle it externally.
			return true;
		}
	}

	template <typename T, typename TGeometryA, typename TGeometryB>
	bool GJKPenetrationSameSpace2(const TGeometryA& A, const TGeometryB& B, T& OutPenetration, TVec3<T>& OutClosestA, TVec3<T>& OutClosestB, TVec3<T>& OutNormal, int32& OutVertexA, int32& OutVertexB, T& OutMaxSupportDelta, const TVec3<T>& InitialDir, const T Epsilon = T(1.e-3), const T EPAEpsilon = T(1.e-2))
	{
		TGJKSimplexData<T> SimplexData;
		T SupportDeltaA = 0;
		T SupportDeltaB = 0;
		T MaxSupportDelta = 0;

		int32& VertexIndexA = OutVertexA;
		int32& VertexIndexB = OutVertexB;

		// Return the support vertex in A-local space for a vector V in A-local space
		auto SupportAFunc = [&A, &SupportDeltaA, &VertexIndexA](const TVec3<T>& V)
		{
			return A.SupportCore(V, A.GetMargin(), &SupportDeltaA, VertexIndexA);
		};

		// Return the support point on B, in B-local space, for a vector V in A-local space
		auto SupportBFunc = [&B, &SupportDeltaB, &VertexIndexB](const TVec3<T>& V)
		{
			return B.SupportCore(V, B.GetMargin(), &SupportDeltaB, VertexIndexB);
		};

		// V and Simplex are in A-local space
		TVec3<T> V = InitialDir;
		TVec3<T> Simplex[4];
		int32 NumVerts = 0;
		T Distance = FLT_MAX;

		TVec3<T> Normal = -V;					// Remember the last good normal (i.e. don't update it if separation goes less than Epsilon and we can no longer normalize)
		bool bIsResult = false;				// True if GJK cannot make any more progress
		bool bIsContact = false;				// True if shapes are within Epsilon or overlapping - GJK cannot provide a solution
		int32 NumIterations = 0;
		const T ThicknessA = A.GetMargin();
		const T ThicknessB = B.GetMargin();
		const T SeparatedDistance = ThicknessA + ThicknessB + Epsilon;
		while (!bIsContact && !bIsResult)
		{
			// If taking too long just stop
			++NumIterations;
			if (CheckGJKIterationLimit(NumIterations, A, B))
			{
				break;
			}

			const TVec3<T> NegV = -V;
			const TVec3<T> SupportA = SupportAFunc(NegV);
			const TVec3<T> SupportB = SupportBFunc(V);
			const TVec3<T> W = SupportA - SupportB;
			MaxSupportDelta = FMath::Max(SupportDeltaA, SupportDeltaB);

			// If we didn't move to at least ConvergedDistance or closer, assume we have reached a minimum
			const T ConvergenceTolerance = 1.e-4f;
			const T ConvergedDistance = (1.0f - ConvergenceTolerance) * Distance;
			const T VW = TVec3<T>::DotProduct(V, W);

			SimplexData.As[NumVerts] = SupportA;
			SimplexData.Bs[NumVerts] = SupportB;
			Simplex[NumVerts++] = W;

			V = SimplexFindClosestToOrigin2(Simplex, NumVerts, SimplexData.Barycentric, SimplexData.As, SimplexData.Bs);
			T NewDistance = V.Size();

			// If we hit this error, we probably have a degenerate simplex that is not being detected in SimplexFindClosestToOrigin2
			CHAOS_COLLISIONERROR_CLOG(V.ContainsNaN(), TEXT("SimplexFindClosestToOrigin2 NaN, NumVerts: %d, Simplex: [[%f, %f, %f], [%f %f %f], [%f %f %f], [%f %f %f]]"), 
				NumVerts, Simplex[0].X, Simplex[0].Y, Simplex[0].Z, Simplex[1].X, Simplex[1].Y, Simplex[1].Z, Simplex[2].X, Simplex[2].Y, Simplex[2].Z, Simplex[3].X, Simplex[3].Y, Simplex[3].Z);
			CHAOS_COLLISIONERROR_ENSURE(!V.ContainsNaN());

			// Are we overlapping or too close for GJK to get a good result?
			bIsContact = (NewDistance < Epsilon);

			// If we did not get closer in this iteration, we are in a degenerate situation
			bIsResult = (NewDistance >= (Distance - Epsilon));

			// If we are not too close, update the separating vector and keep iterating
			// If we are too close we drop through to EPA, but if EPA determines the shapes are actually separated 
			// after all, we will wend up using the normal from the previous loop
			if (!bIsContact)
			{
				V /= NewDistance;
				Normal = -V;
			}
			Distance = NewDistance;
		}

		// Save the warm-start data (not much to store since we updated most of it in the loop above)
		SimplexData.NumVerts = NumVerts;

		if (bIsContact)
		{
			// We did not converge or we detected an overlap situation, so run EPA to get contact data
			// Rebuild the simplex into a variable sized array - EPA can generate any mumber of vertices
			TArray<TVec3<T>> VertsA;
			TArray<TVec3<T>> VertsB;
			VertsA.Reserve(8);
			VertsB.Reserve(8);
			for (int i = 0; i < NumVerts; ++i)
			{
				VertsA.Add(SimplexData.As[i]);
				VertsB.Add(SimplexData.Bs[i]);
			}

			T Penetration;
			TVec3<T> MTD, ClosestA, ClosestB;
			const EEPAResult EPAResult = EPA<T>(VertsA, VertsB, SupportAFunc, SupportBFunc, Penetration, MTD, ClosestA, ClosestB, EPAEpsilon);

			switch (EPAResult)
			{
			case EEPAResult::MaxIterations:
				// Possibly a solution but with unknown error. Just return the last EPA state. 
				// Fall through...
			case EEPAResult::Ok:
				// EPA has a solution - return it now
				OutNormal = MTD;
				OutPenetration = Penetration + ThicknessA + ThicknessB;
				OutClosestA = ClosestA + MTD * ThicknessA;
				OutClosestB = ClosestB - MTD * ThicknessB;
				OutMaxSupportDelta = MaxSupportDelta;
				return true;
			case EEPAResult::BadInitialSimplex:
				// The origin is outside the simplex. Must be a touching contact and EPA setup will have calculated the normal
				// and penetration but we keep the position generated by GJK.
				Normal = MTD;
				Distance = -Penetration;
				//UE_LOGF(LogChaos, Warning, "EPA Touching Case");
				break;
			case EEPAResult::Degenerate:
				// We hit a degenerate simplex condition and could not reach a solution so use whatever near touching point GJK came up with.
				// The result from EPA under these circumstances is not usable.
				//UE_LOGF(LogChaos, Warning, "EPA Degenerate Case");
				break;
			}
		}

		// @todo(chaos): handle the case where EPA hits a degenerate triangle in the simplex.
		// Currently we return a touching contact with the last position and normal from GJK. We could run SAT instead.

		// GJK converged or we have a touching contact
		{
			TVec3<T> ClosestA(0);
			TVec3<T> ClosestB(0);
			for (int i = 0; i < NumVerts; ++i)
			{
				ClosestA += SimplexData.As[i] * SimplexData.Barycentric[i];
				ClosestB += SimplexData.Bs[i] * SimplexData.Barycentric[i];
			}

			OutPenetration = ThicknessA + ThicknessB - Distance;
			OutClosestA = ClosestA + Normal * ThicknessA;
			OutClosestB = ClosestB - Normal * ThicknessB;
			OutNormal = Normal;
			OutMaxSupportDelta = MaxSupportDelta;

			// @todo(chaos): we should pass back failure for the degenerate case so we can decide how to handle it externally.
			return true;
		}
	}

	template <bool bNegativePenetrationAllowed = false, typename T>
	bool GJKPenetrationImpl(const FGeomGJKHelper& A, const FGeomGJKHelper& B, const TRigidTransform<T, 3>& BToATM, T& OutPenetration, TVec3<T>& OutClosestA, TVec3<T>& OutClosestB, TVec3<T>& OutNormal, int32& OutClosestVertexIndexA, int32& OutClosestVertexIndexB, const T InThicknessA = 0.0f, const T InThicknessB = 0.0f, const TVector<T, 3>& InitialDir = TVector<T, 3>(1, 0, 0), const T Epsilon = 1.e-3f)
	{
		int32 VertexIndexA = INDEX_NONE;
		int32 VertexIndexB = INDEX_NONE;

		const TRotation<T, 3> AToBRotation = BToATM.GetRotation().Inverse();

		//todo: refactor all of these similar functions
		TVector<T, 3> V = -InitialDir;
		if (V.SafeNormalize() == 0)
		{
			V = TVec3<T>(-1, 0, 0);
		}

		TVec3<T> As[4];
		TVec3<T> Bs[4];

		FSimplex SimplexIDs;
		SimplexIDs.NumVerts = 0;                // Initialization not needed, but compiler warns on uninitialized access to As/Bs
		TVector<T, 3> Simplex[4];
		T Barycentric[4] = { -1,-1,-1,-1 };		// Initialization not needed, but compiler warns
		TVec3<T> Normal = -V;					// Remember the last good normal (i.e. don't update it if separation goes less than Epsilon and we can no longer normalize)
		bool bIsDegenerate = false;				// True if GJK cannot make any more progress
		bool bIsContact = false;				// True if shapes are within Epsilon or overlapping - GJK cannot provide a solution
		int NumIterations = 0;
		T Distance = FLT_MAX;
		const T ThicknessA = InThicknessA + A.GetMargin();
		const T ThicknessB = InThicknessB + B.GetMargin();
		const T SeparatedDistance = ThicknessA + ThicknessB + Epsilon;
		while (!bIsContact && !bIsDegenerate)
		{
			if (!ensure(NumIterations++ < 32))	//todo: take this out
			{
				break;	//if taking too long just stop. This should never happen
			}
			const TVector<T, 3> NegV = -V;
			const TVector<T, 3> SupportA = A.SupportFunction(NegV, VertexIndexA);
			const TVector<T, 3> SupportB = B.SupportFunction(BToATM, AToBRotation, V, VertexIndexB);
			const TVector<T, 3> W = SupportA - SupportB;

			const T VW = TVector<T, 3>::DotProduct(V, W);
			if (!bNegativePenetrationAllowed && (VW > SeparatedDistance))
			{
				// We are separated and don't care about the distance - we can stop now
				return false;
			}

			// If we didn't move to at least ConvergedDistance or closer, assume we have reached a minimum
			const T ConvergenceTolerance = 1.e-4f;
			const T ConvergedDistance = (1.0f - ConvergenceTolerance) * Distance;
			if (VW > ConvergedDistance)
			{
				// We have reached a solution - use the results from the last iteration
				break;
			}

			SimplexIDs[SimplexIDs.NumVerts] = SimplexIDs.NumVerts;
			As[SimplexIDs.NumVerts] = SupportA;
			Bs[SimplexIDs.NumVerts] = SupportB;
			Simplex[SimplexIDs.NumVerts++] = W;

			V = SimplexFindClosestToOrigin(Simplex, SimplexIDs, Barycentric, As, Bs);
			T NewDistance = V.Size();

			// Are we overlapping or too close for GJK to get a good result?
			bIsContact = (NewDistance < Epsilon);

			// If we did not get closer in this iteration, we are in a degenerate situation
			bIsDegenerate = (NewDistance >= Distance);

			if (!bIsContact)
			{
				V /= NewDistance;
				Normal = -V;
			}
			Distance = NewDistance;
		}

		if (bIsContact)
		{
			// We did not converge or we detected an overlap situation, so run EPA to get contact data
			TArray<TVec3<T>> VertsA;
			TArray<TVec3<T>> VertsB;
			VertsA.Reserve(8);
			VertsB.Reserve(8);
			for (int i = 0; i < SimplexIDs.NumVerts; ++i)
			{
				VertsA.Add(As[i]);
				VertsB.Add(Bs[i]);
			}

			struct FAHelper
			{
				const FGeomGJKHelper& Geom;
				int32* VertexIndex;

				FAHelper(const FGeomGJKHelper& Geom, int32& VertexIndex) : Geom(Geom), VertexIndex(&VertexIndex) {}

				TVec3<T> operator()(const TVec3<T>& V) const { return Geom.SupportFunction(V, *VertexIndex); }
			};

			struct FBHelper
			{
				const FGeomGJKHelper& Geom;
				int32* VertexIndex;
				const FRigidTransform3& BToATM;
				const FRotation3& AToBRotation;

				TVec3<T> operator()(const TVec3<T>& V) const { return Geom.SupportFunction(BToATM, AToBRotation, V, *VertexIndex); }
			};

			FAHelper AHelper(A, VertexIndexA);
			FBHelper BHelper = { B, &VertexIndexB, BToATM, AToBRotation };

			T Penetration;
			TVec3<T> MTD, ClosestA, ClosestBInA;
			const EEPAResult EPAResult = EPA<T>(VertsA, VertsB, AHelper, BHelper, Penetration, MTD, ClosestA, ClosestBInA);
			
			switch (EPAResult)
			{
			case EEPAResult::MaxIterations:
				// Possibly a solution but with unknown error. Just return the last EPA state. 
				// Fall through...
			case EEPAResult::Ok:
				// EPA has a solution - return it now
				OutNormal = MTD;
				OutPenetration = Penetration + ThicknessA + ThicknessB;
				OutClosestA = ClosestA + MTD * ThicknessA;
				OutClosestB = ClosestBInA - MTD * ThicknessB;
				OutClosestVertexIndexA = VertexIndexA;
				OutClosestVertexIndexB = VertexIndexB;
				return true;
			case EEPAResult::BadInitialSimplex:
				// The origin is outside the simplex. Must be a touching contact and EPA setup will have calculated the normal
				// and penetration but we keep the position generated by GJK.
				Normal = MTD;
				Distance = -Penetration;
				break;
			case EEPAResult::Degenerate:
				// We hit a degenerate simplex condition and could not reach a solution so use whatever near touching point GJK came up with.
				// The result from EPA under these circumstances is not usable.
				//UE_LOGF(LogChaos, Warning, "EPA Degenerate Case");
				break;
			}
		}

		// @todo(chaos): handle the case where EPA hit a degenerate triangle in the simplex.
		// Currently we return a touching contact with the last position and normal from GJK. We could run SAT instead.

		// GJK converged or we have a touching contact
		{
			TVector<T, 3> ClosestA(0);
			TVector<T, 3> ClosestBInA(0);
			for (int i = 0; i < SimplexIDs.NumVerts; ++i)
			{
				ClosestA += As[i] * Barycentric[i];
				ClosestBInA += Bs[i] * Barycentric[i];
			}

			OutNormal = Normal;

			T Penetration = ThicknessA + ThicknessB - Distance;
			OutPenetration = Penetration;
			OutClosestA = ClosestA + Normal * ThicknessA;
			OutClosestB = ClosestBInA - Normal * ThicknessB;
			OutClosestVertexIndexA = VertexIndexA;
			OutClosestVertexIndexB = VertexIndexB;

			// If we don't care about separation distance/normal, the return value is true if we are overlapping, false otherwise.
			// If we do care about seperation distance/normal, the return value is true if we found a solution.
			// @todo(chaos): we should pass back failure for the degenerate case so we can decide how to handle it externally.
			return (bNegativePenetrationAllowed || (Penetration >= 0.0f));
		}
	}

	// Calculate the penetration depth (or separating distance) of two geometries.
	//
	// Set bNegativePenetrationAllowed to false (default) if you do not care about the normal and distance when the shapes are separated. The return value
	// will be false if the shapes are separated, and the function will be faster because it does not need to determine the closest point.
	// If the shapes are overlapping, the function will return true and populate the output parameters with the contact information.
	//
	// Set bNegativePenetrationAllowed to true if you need to know the closest point on the shapes, even when they are separated. In this case,
	// we need to iterate to find the best solution even when objects are separated which is more expensive. The return value will be true as long 
	// as the algorithm was able to find a solution (i.e., the return value is not related to whether the shapes are overlapping) and the output 
	// parameters will be populated with the contact information.
	//
	// In all cases, if the function returns false the output parameters are undefined.
	//
	// OutClosestA and OutClosestB are the closest or deepest-penetrating points on the two core geometries, both in the space of A and ignoring the margin.
	//
	// Epsilon is the separation at which GJK considers the objects to be in contact or penetrating and then runs EPA. If this is
	// too small, then the renormalization of the separating vector can lead to arbitrarily wrong normals for almost-touching objects.
	//
	// NOTE: OutPenetration is the penetration including the Thickness (i.e., the actual penetration depth), but the closest points
	// returned are on the core shapes (i.e., ignoring the Thickness). If you want the closest positions on the shape surface (including
	// the Thickness) use GJKPenetration().
	//
	template <bool bNegativePenetrationAllowed = false, typename T, typename TGeometryA, typename TGeometryB>
	bool GJKPenetration(const TGeometryA& A, const TGeometryB& B, const TRigidTransform<T, 3>& BToATM, T& OutPenetration, TVec3<T>& OutClosestA, TVec3<T>& OutClosestB, TVec3<T>& OutNormal, int32& OutClosestVertexIndexA, int32& OutClosestVertexIndexB, const T InThicknessA = 0.0f, const T InThicknessB = 0.0f, const TVector<T, 3>& InitialDir = TVector<T, 3>(1, 0, 0), const T Epsilon = 1.e-3f)
	{
		return GJKPenetrationImpl<bNegativePenetrationAllowed, T>(A, B, BToATM, OutPenetration, OutClosestA, OutClosestB, OutNormal, OutClosestVertexIndexA, OutClosestVertexIndexB, InThicknessA, InThicknessB, InitialDir, Epsilon);
	}
	/** Sweeps one geometry against the other
	 @A The first geometry
	 @B The second geometry
	 @StartTM B's starting configuration in A's local space
	 @RayDir The ray's direction (normalized)
	 @RayLength The ray's length
	 @OutTime The time along the ray when the objects first overlap
	 @OutPosition The first point of impact (in A's local space) when the objects first overlap. Invalid if time of impact is 0
	 @OutNormal The impact normal (in A's local space) when the objects first overlap. Invalid if time of impact is 0
	 @ThicknessA The amount of geometry inflation for Geometry A (for example if the surface distance of two geometries with thickness 0 would be 2, a thickness of 0.5 would give a distance of 1.5)
	 @InitialDir The first direction we use to search the CSO
	 @ThicknessB The amount of geometry inflation for Geometry B (for example if the surface distance of two geometries with thickness 0 would be 2, a thickness of 0.5 would give a distance of 1.5)
	 @return True if the geometries overlap during the sweep, False otherwise 
	 @note If A overlaps B at the start of the ray ("initial overlap" condition) then this function returns true, and sets OutTime = 0, but does not set any other output variables.
	 */

	template <typename T, typename TGeometryA, typename TGeometryB>
	bool GJKRaycast(const TGeometryA& A, const TGeometryB& B, const TRigidTransform<T, 3>& StartTM, const TVector<T, 3>& RayDir, const T RayLength,
		T& OutTime, TVector<T, 3>& OutPosition, TVector<T, 3>& OutNormal, const T ThicknessA = 0, const TVector<T, 3>& InitialDir = TVector<T, 3>(1, 0, 0), const T ThicknessB = 0)
	{
		ensure(FMath::IsNearlyEqual(RayDir.SizeSquared(), (FReal)1, (FReal)UE_KINDA_SMALL_NUMBER));
		ensure(RayLength >= 0);
		check(A.IsConvex() && B.IsConvex());
		const TVector<T, 3> StartPoint = StartTM.GetLocation();
		int32 VertexIndexA = INDEX_NONE;
		int32 VertexIndexB = INDEX_NONE;

		TVector<T, 3> Simplex[4] = { TVector<T,3>(0), TVector<T,3>(0), TVector<T,3>(0), TVector<T,3>(0) };
		TVector<T, 3> As[4] = { TVector<T,3>(0), TVector<T,3>(0), TVector<T,3>(0), TVector<T,3>(0) };
		TVector<T, 3> Bs[4] = { TVector<T,3>(0), TVector<T,3>(0), TVector<T,3>(0), TVector<T,3>(0) };

		T Barycentric[4] = { -1,-1,-1,-1 };	//not needed, but compiler warns

		FSimplex SimplexIDs;
		const TRotation<T, 3> BToARotation = StartTM.GetRotation();
		const TRotation<T, 3> AToBRotation = BToARotation.Inverse();
		TVector<T, 3> SupportA = A.Support(InitialDir, ThicknessA, VertexIndexA);	//todo: use Thickness on quadratic geometry
		As[0] = SupportA;

		const TVector<T, 3> InitialDirInB = AToBRotation * (-InitialDir);
		const TVector<T, 3> InitialSupportBLocal = B.Support(InitialDirInB, ThicknessB, VertexIndexB);
		TVector<T, 3> SupportB = BToARotation * InitialSupportBLocal;
		Bs[0] = SupportB;

		T Lambda = 0;
		TVector<T, 3> X = StartPoint;
		TVector<T, 3> Normal(0);
		TVector<T, 3> V = X - (SupportA - SupportB);

		bool bTerminate;
		bool bNearZero = false;
		bool bDegenerate = false;
		int NumIterations = 0;
		T GJKPreDist2 = TNumericLimits<T>::Max();
		do
		{
			//if (!ensure(NumIterations++ < 32))	//todo: take this out
			if (!(NumIterations++ < 32))	//todo: take this out
			{
				break;	//if taking too long just stop. This should never happen
			}

			SupportA = A.Support(V, ThicknessA, VertexIndexA);	//todo: add thickness to quadratic geometry to avoid quadratic vs quadratic when possible
			const TVector<T, 3> VInB = AToBRotation * (-V);
			const TVector<T, 3> SupportBLocal = B.Support(VInB, ThicknessB, VertexIndexB);
			SupportB = BToARotation * SupportBLocal;
			const TVector<T, 3> P = SupportA - SupportB;
			const TVector<T, 3> W = X - P;
			SimplexIDs[SimplexIDs.NumVerts] = SimplexIDs.NumVerts;	//is this needed?
			As[SimplexIDs.NumVerts] = SupportA;
			Bs[SimplexIDs.NumVerts] = SupportB;

			const T VDotW = TVector<T, 3>::DotProduct(V, W);
			if (VDotW > 0)
			{
				const T VDotRayDir = TVector<T, 3>::DotProduct(V, RayDir);
				if (VDotRayDir >= 0)
				{
					return false;
				}

				const T PreLambda = Lambda;	//use to check for no progress
				// @todo(ccaulfield): this can still overflow - the comparisons against zero above should be changed (though not sure to what yet)
				Lambda = Lambda - VDotW / VDotRayDir;
				if (Lambda > PreLambda)
				{
					if (Lambda > RayLength)
					{
						return false;
					}

					const TVector<T, 3> OldX = X;
					X = StartPoint + Lambda * RayDir;
					Normal = V;

					//Update simplex from (OldX - P) to (X - P)
					const TVector<T, 3> XMinusOldX = X - OldX;
					Simplex[0] += XMinusOldX;
					Simplex[1] += XMinusOldX;
					Simplex[2] += XMinusOldX;
					Simplex[SimplexIDs.NumVerts++] = X - P;

					GJKPreDist2 = TNumericLimits<T>::Max();	//translated origin so restart gjk search
				}
			}
			else
			{
				Simplex[SimplexIDs.NumVerts++] = W;	//this is really X - P which is what we need for simplex computation
			}

			V = SimplexFindClosestToOrigin(Simplex, SimplexIDs, Barycentric, As, Bs);

			T NewDist2 = V.SizeSquared();	//todo: relative error
			bNearZero = NewDist2 < 1e-6;
			bDegenerate = NewDist2 >= GJKPreDist2;
			GJKPreDist2 = NewDist2;
			bTerminate = bNearZero || bDegenerate;

		} while (!bTerminate);

		OutTime = Lambda;

		if (Lambda > 0)
		{
			OutNormal = Normal.GetUnsafeNormal();
			TVector<T, 3> ClosestA(0);
			TVector<T, 3> ClosestB(0);

			for (int i = 0; i < SimplexIDs.NumVerts; ++i)
			{
				ClosestB += Bs[i] * Barycentric[i];
			}
			const TVector<T, 3> ClosestLocal = ClosestB;

			OutPosition = StartPoint + RayDir * Lambda + ClosestLocal;
		}

		return true;
	}
	/** Sweeps one geometry against the other
	 @A The first geometry
	 @B The second geometry
	 @BToARotation B's starting rotation in A's local space
	 @StartPoint B's starting position in A's local space
	 @RayDir The ray's direction (normalized)
	 @RayLength The ray's length
	 @OutTime The time along the ray when the objects first overlap
	 @OutPosition The first point of impact (in A's local space) when the objects first overlap. Invalid if time of impact is 0
	 @OutNormal The impact normal (in A's local space) when the objects first overlap. Invalid if time of impact is 0
	 @bComputeMTD If there is an initial overlap, this will compute the output info from the overlap info.
	 @InitialDir The first direction we use to search the CSO
	 @return True if the geometries overlap during the sweep, False otherwise
	 @note If A overlaps B at the start of the ray ("initial overlap" condition) then this function returns true, and sets OutTime = 0, but does not set any other output variables unless bComputeMTD is true.
	 */
	template<typename T, EGJKTestSpace TestSpace = EGJKTestSpace::ALocalSpace>
	bool GJKRaycast2ImplSimd(const FGeomGJKHelperSIMD& RESTRICT A, const FGeomGJKHelperSIMD& RESTRICT B, const T& BToARotation, const T& StartPoint, const T& RayDir,
		FRealSingle RayLength, FRealSingle& OutTime, T& OutPosition, T& OutNormal, bool bComputeMTD, const T& InitialDir)
	{
		ensure(RayLength >= 0);

		// Margin selection logic: we only need a small margin for sweeps since we only move the sweeping object
		// to the point where it just touches.
		// Spheres and Capsules: always use the core shape and full "margin" because it represents the radius
		// Sphere/Capsule versus OtherShape: no margin on other
		// OtherShape versus OtherShape: use margin of the smaller shape, zero margin on the other
		const FRealSingle RadiusA = static_cast<FRealSingle>(A.GetRadius());
		const FRealSingle RadiusB = static_cast<FRealSingle>(B.GetRadius());
		const bool bHasRadiusA = RadiusA > 0;
		const bool bHasRadiusB = RadiusB > 0;

		// The sweep margins if required. Only one can be non-zero (we keep the smaller one)
		const FRealSingle SweepMarginScale = 0.05f;
		const bool bAIsSmallest = A.GetMargin() < B.GetMargin();
		const FRealSingle SweepMarginA = (bHasRadiusA || bHasRadiusB) ? 0.0f : (bAIsSmallest ? SweepMarginScale * static_cast<FRealSingle>(A.GetMargin()) : 0.0f);
		const FRealSingle SweepMarginB = (bHasRadiusA || bHasRadiusB) ? 0.0f : (bAIsSmallest ? 0.0f : SweepMarginScale * static_cast<FRealSingle>(B.GetMargin()));

		// Net margin (note: both SweepMargins are zero if either Radius is non-zero, and only one SweepMargin can be non-zero)
		A.Margin = RadiusA + SweepMarginA;
		B.Margin = RadiusB + SweepMarginB;

		const T MarginASimd = T(VectorLoadFloat1(&A.Margin));
		const T MarginBSimd = T(VectorLoadFloat1(&B.Margin));

		T Simplex[4] = { TVectorZero<T>(), TVectorZero<T>(), TVectorZero<T>(), TVectorZero<T>() };
		T As[4] = { TVectorZero<T>(), TVectorZero<T>(), TVectorZero<T>(), TVectorZero<T>() };
		T Bs[4] = { TVectorZero<T>(), TVectorZero<T>(), TVectorZero<T>(), TVectorZero<T>() };

		T Barycentric = TVectorZero<T>();

		T Inflation = VectorAdd(MarginASimd, MarginBSimd);
		const T Eps2Simd = TMakeVectorRegister<T>(1e-6f, 1e-6f, 1e-6f, 1e-6f);
		const T Inflation2Simd = VectorMultiplyAdd(Inflation, Inflation, Eps2Simd);

		const T RayLengthSimd = TMakeVectorRegister<T>(RayLength, RayLength, RayLength, RayLength);

		int32 NumVerts = 0;

		T AToBRotation = VectorQuaternionInverse(BToARotation);

		T SupportA, SupportB;
		TGJKSupportHelperSIMD<T, TestSpace>::Support(A, B, InitialDir, AToBRotation, BToARotation, SupportA, SupportB);
		As[0] = SupportA;
		Bs[0] = SupportB;

		T Lambda = TVectorZero<T>();
		T X = StartPoint;
		T V = VectorSubtract(X, VectorSubtract(SupportA, SupportB));
		T Normal = MakeVectorRegisterFloat(0.f, 0.f, 1.f, 0.f);

		const T InitialPreDist2Simd = VectorDot3(V, V);

		FRealSingle InitialPreDist2;
		VectorStoreFloat1(InitialPreDist2Simd, &InitialPreDist2);

		FRealSingle Inflation2;
		VectorStoreFloat1(Inflation2Simd, &Inflation2); 

		constexpr FRealSingle Eps2 = 1e-6f;

		//mtd needs to find closest point even in inflation region, so can only skip if we found the closest points
		bool bCloseEnough = InitialPreDist2 < Inflation2 && (!bComputeMTD || InitialPreDist2 < Eps2);
		bool bDegenerate = false;
		bool bTerminate = bCloseEnough;
		bool bInflatedCloseEnough = bCloseEnough;
		int NumIterations = 0;
		constexpr T LimitMax = TMakeVectorRegisterConstant<T>(TNumericLimits<FRealSingle>::Max(), TNumericLimits<FRealSingle>::Max(), TNumericLimits<FRealSingle>::Max(), TNumericLimits<FRealSingle>::Max());
		T GJKPreDist2 = LimitMax;

		while (!bTerminate)
		{
			if (!(NumIterations++ < 32))	//todo: take this out
			{
				break;	//if taking too long just stop. This should never happen
			}

			V = VectorNormalizeAccurate(V);

			TGJKSupportHelperSIMD<T, TestSpace>::Support(A, B, V, AToBRotation, BToARotation, SupportA, SupportB);
			T P = VectorSubtract(SupportA, SupportB);
			T W = VectorSubtract(X, P);


			As[NumVerts] = SupportA;
			Bs[NumVerts] = SupportB;

			const T VDotW = VectorDot3(V, W);

			T VDotWGTInflationSimd = VectorCompareGT(VDotW, Inflation);

			if (::VectorMaskBits(VDotWGTInflationSimd))
			{
				const T VDotRayDir = VectorDot3(V, RayDir);
				T VDotRayDirGEZero = VectorCompareGE(VDotRayDir, TVectorZero<T>());

				if (::VectorMaskBits(VDotRayDirGEZero))
				{
					return false;
				}

				const T PreLambda = Lambda;	//use to check for no progress
				// @todo(ccaulfield): this can still overflow - the comparisons against zero above should be changed (though not sure to what yet)
				Lambda = VectorSubtract(Lambda, VectorDivide(VectorSubtract(VDotW, Inflation), VDotRayDir));
				T LambdaGTPreLambda = VectorCompareGT(Lambda, PreLambda);
				if (::VectorMaskBits(LambdaGTPreLambda))
				{
					T LambdaGTRayLength = VectorCompareGT(Lambda, RayLengthSimd);
					if (::VectorMaskBits(LambdaGTRayLength))
					{
						return false;
					}

					const T OldX = X;
					X = VectorMultiplyAdd(Lambda, RayDir, StartPoint);
					Normal = V;

					//Update simplex with the new X
					// As and Bs should have been correcly updated by enabling the flag CalculatExtraInformation
					// inside VectorSimplexFindClosestToOrigin
					Simplex[0] = VectorSubtract(X, VectorSubtract(As[0], Bs[0]));
					Simplex[1] = VectorSubtract(X, VectorSubtract(As[1], Bs[1]));
					Simplex[2] = VectorSubtract(X, VectorSubtract(As[2], Bs[2]));
					Simplex[NumVerts++] = VectorSubtract(X, P);

					GJKPreDist2 = LimitMax; //translated origin so restart gjk search
					bInflatedCloseEnough = false;
				}
			}
			else
			{
				Simplex[NumVerts++] = W;	//this is really X - P which is what we need for simplex computation
			}

			if (bInflatedCloseEnough && ::VectorMaskBits(VectorCompareGE(VDotW, TVectorZero<T>())))
			{
				//Inflated shapes are close enough, but we want MTD so we need to find closest point on core shape
				const T VDotW2 = VectorDot3(VDotW, VDotW);
				bCloseEnough = static_cast<bool>(::VectorMaskBits(VectorCompareGE(VectorAdd(Eps2Simd, VDotW), GJKPreDist2)));
			}

			if (!bCloseEnough)
			{

				V = VectorSimplexFindClosestToOrigin<T>(Simplex, NumVerts, Barycentric, As, Bs);

				T NewDist2 = VectorDot3(V, V);	//todo: relative error
				bCloseEnough = static_cast<bool>(::VectorMaskBits(VectorCompareGT(Inflation2Simd, NewDist2)));
				bDegenerate = static_cast<bool>(::VectorMaskBits(VectorCompareGE(NewDist2, GJKPreDist2)));
				GJKPreDist2 = NewDist2;

				if (bComputeMTD && bCloseEnough)
				{
					const T LambdaEqZero = ::VectorCompareEQ(Lambda, TVectorZero<T>());
					const T InGJKPreDist2GTEps2 = VectorCompareGT(GJKPreDist2, Eps2Simd);
					const T Inflation22GTEps2 = VectorCompareGT(Inflation2Simd, Eps2Simd);

					const bool Is4GTNumVerts = 4 > NumVerts;

					const T IsInflatCloseEnough = VectorBitwiseAnd(LambdaEqZero, VectorBitwiseAnd(InGJKPreDist2GTEps2, Inflation22GTEps2));

					// Leaving the original code, to explain the logic there
					//if (bComputeMTD && bCloseEnough && Lambda == 0 && GJKPreDist2 > 1e-6 && Inflation2 > 1e-6 && NumVerts < 4)
					//{
					//	bCloseEnough = false;
					//	bInflatedCloseEnough = true;
					//}
					bInflatedCloseEnough = static_cast<bool>(::VectorMaskBits(IsInflatCloseEnough)) && Is4GTNumVerts;
					bCloseEnough = !bInflatedCloseEnough;
				}
			}
			else
			{
				//It must be that we want MTD and we can terminate. However, we must make one final call to fixup the simplex
				V = VectorSimplexFindClosestToOrigin<T>(Simplex, NumVerts, Barycentric, As, Bs);
			}
			bTerminate = bCloseEnough || bDegenerate;
		}
		VectorStoreFloat1(Lambda, &OutTime);

		if (OutTime > 0)
		{
			OutNormal = Normal;
			T ClosestB = TVectorZero<T>();

			T Barycentrics[4];
			Barycentrics[0] = VectorSwizzle(Barycentric, 0, 0, 0, 0);
			Barycentrics[1] = VectorSwizzle(Barycentric, 1, 1, 1, 1);
			Barycentrics[2] = VectorSwizzle(Barycentric, 2, 2, 2, 2);
			Barycentrics[3] = VectorSwizzle(Barycentric, 3, 3, 3, 3);


			const T  ClosestB1 = VectorMultiplyAdd(Bs[0], Barycentrics[0], ClosestB);
			const T  ClosestB2 = VectorMultiplyAdd(Bs[1], Barycentrics[1], ClosestB1);
			const T  ClosestB3 = VectorMultiplyAdd(Bs[2], Barycentrics[2], ClosestB2);
			const T  ClosestB4 = VectorMultiplyAdd(Bs[3], Barycentrics[3], ClosestB3);

			ClosestB = NumVerts == 0 ? ClosestB : ClosestB4;
			ClosestB = NumVerts == 1 ? ClosestB1 : ClosestB;
			ClosestB = NumVerts == 2 ? ClosestB2 : ClosestB;
			ClosestB = NumVerts == 3 ? ClosestB3 : ClosestB;

			const T ClosestLocal = VectorNegateMultiplyAdd(Normal, MarginBSimd, ClosestB);

			OutPosition = VectorAdd(VectorMultiplyAdd(RayDir, Lambda, StartPoint), ClosestLocal);

		}
		else if (bComputeMTD)
		{
			// If Inflation == 0 we would expect GJKPreDist2 to be 0
			// However, due to precision we can still end up with GJK failing.
			// When that happens fall back on EPA
			T InflationGTZero = VectorCompareGT(Inflation, TVectorZero<T>());
			T InGJKPreDist2GTEps2 = VectorCompareGT(GJKPreDist2, Eps2Simd);
			T LimitMaxGTInGJKPreDist2 = VectorCompareGT(LimitMax, GJKPreDist2);
			T IsDone = VectorBitwiseAnd(InflationGTZero, VectorBitwiseAnd(InGJKPreDist2GTEps2, LimitMaxGTInGJKPreDist2));

			T GJKClosestA = TVectorZero<T>();
			T GJKClosestB = TVectorZero<T>();

			if (NumIterations)
			{
				T Barycentrics[4];
				Barycentrics[0] = VectorSwizzle(Barycentric, 0, 0, 0, 0);
				Barycentrics[1] = VectorSwizzle(Barycentric, 1, 1, 1, 1);
				Barycentrics[2] = VectorSwizzle(Barycentric, 2, 2, 2, 2);
				Barycentrics[3] = VectorSwizzle(Barycentric, 3, 3, 3, 3);

				for (int i = 0; i < NumVerts; ++i)
				{
					GJKClosestA = VectorMultiplyAdd(As[i], Barycentrics[i], GJKClosestA);
					GJKClosestB = VectorMultiplyAdd(Bs[i], Barycentrics[i], GJKClosestB);
				}
			}
			else
			{
				//didn't even go into gjk loop
				GJKClosestA = As[0];
				GJKClosestB = Bs[0];
			}

			if (::VectorMaskBits(IsDone))
			{
				const T ClosestBInA = VectorAdd(StartPoint, GJKClosestB);
				const T InGJKPreDist = VectorSqrt(GJKPreDist2);
				OutNormal = VectorNormalizeAccurate(V);

				T Penetration = VectorSubtract(VectorAdd(MarginASimd, MarginBSimd), InGJKPreDist);
				Penetration = VectorMin(Penetration, LimitMax);
				Penetration = VectorMax(Penetration, TVectorZero<T>());

				const T ClosestLocal = VectorNegateMultiplyAdd(OutNormal, MarginBSimd, GJKClosestB);

				OutPosition = VectorAdd(VectorMultiplyAdd(OutNormal, Penetration, StartPoint), ClosestLocal);
				Penetration = VectorNegate(Penetration);
				VectorStoreFloat1(Penetration, &OutTime);
			}
			else
			{
				if (NumIterations)
				{
					TArray<VectorRegister4Float> VertsA;
					TArray<VectorRegister4Float> VertsB;

					VertsA.Reserve(8);
					VertsB.Reserve(8);

					for (int i = 0; i < NumVerts; ++i)
					{
						VertsA.Add(TMakeVectorRegisterFloatFromDouble<T>(As[i]));
						VertsB.Add(TMakeVectorRegisterFloatFromDouble<T>(VectorAdd(Bs[i], X)));
					}

					TSupportBAtOriginHelperSIMD<TestSpace> SupportBAtOrigin(B, TMakeVectorRegisterFloatFromDouble<T>(AToBRotation), TMakeVectorRegisterFloatFromDouble<T>(BToARotation), TMakeVectorRegisterFloatFromDouble<T>(StartPoint));
					VectorRegister4Float Penetration;
					VectorRegister4Float MTD, ClosestA, ClosestBInA;
					const EEPAResult EPAResult = VectorEPA(VertsA, VertsB, A, SupportBAtOrigin, Penetration, MTD, ClosestA, ClosestBInA);
					if ((EPAResult == EEPAResult::Ok) || (EPAResult == EEPAResult::MaxIterations))
					{
						OutNormal = MTD;
						VectorStoreFloat1(Penetration, &OutTime);
						OutTime = -OutTime - (A.Margin + B.Margin);
						OutPosition = VectorMultiplyAdd(OutNormal, MarginASimd, ClosestA);
					}
					else if (EPAResult == EEPAResult::BadInitialSimplex)
					{
						// The origin is outside the simplex. Must be a touching contact and EPA setup will have calculated the normal
						// and penetration but we keep the position generated by GJK.
						OutNormal = MTD;
						VectorStoreFloat1(Penetration, &OutTime);
						OutTime = -OutTime - (A.Margin + B.Margin);
						OutPosition = VectorMultiplyAdd(OutNormal, MarginASimd, GJKClosestA);
					}
					else if (EPAResult == EEPAResult::Degenerate)
					{
						// Assume touching hit and use the last GJK result because
						// EPA does not necessarily return a good normal when GJK provides it with 
						// a near-degenerate simplex that does not contain the origin (it can reject
						// the nearest face and return an arbitrarily bad normal)
						OutTime = -(A.Margin + B.Margin);
						OutNormal = Normal;
						OutPosition = VectorMultiplyAdd(OutNormal, MarginASimd, GJKClosestA);
					}
					else // EEPAResult::NoValidContact
					{
						return false;
					}
				}
				else
				{
					//didn't even go into gjk loop, touching hit
					OutTime = -(A.Margin + B.Margin);
					OutNormal = TMakeVectorRegister<T>(0.0f, 0.0f, 1.0f, 0.0f);
					OutPosition = VectorMultiplyAdd(OutNormal, MarginASimd, As[0]);
				}
			}
		}
		else
		{
			// Initial overlap without MTD. These properties are not valid, but assigning them anyway so they don't contain NaNs and cause issues in invoking code.
			OutNormal = TMakeVectorRegister<T>(0.0f, 0.0f, 1.0f, 0.0f);
			OutPosition = TMakeVectorRegister<T>(0.0f, 0.0f, 0.0f, 0.0f);
		}

		return true;
	}

	inline bool GJKRaycast2Impl(const FGeomGJKHelperSIMD& A, const FGeomGJKHelperSIMD& B, const TRigidTransform<FReal, 3>& StartTM, const TVector<FReal, 3>& RayDir, const FReal RayLength, 
		FReal& OutTime, TVector<FReal, 3>& OutPosition, TVector<FReal, 3>& OutNormal, const FReal GivenThicknessA, bool bComputeMTD, const TVector<FReal, 3>& InitialDir, const FReal GivenThicknessB)
	{
		const UE::Math::TQuat<FReal>& RotationDouble = StartTM.GetRotation();
		VectorRegister4Float Rotation = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(RotationDouble.X, RotationDouble.Y, RotationDouble.Z, RotationDouble.W));

		const UE::Math::TVector<FReal>& TranslationDouble = StartTM.GetTranslation();
		const VectorRegister4Float Translation = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(TranslationDouble.X, TranslationDouble.Y, TranslationDouble.Z, 0.0));

		// Normalize rotation
		Rotation = VectorNormalizeSafe(Rotation, GlobalVectorConstants::Float0001);

		const VectorRegister4Float InitialDirSimd = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(InitialDir[0], InitialDir[1], InitialDir[2], 0.0));
		const VectorRegister4Float RayDirSimd = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(RayDir[0], RayDir[1], RayDir[2], 0.0));

		FRealSingle OutTimeFloat = 0.0f;
		VectorRegister4Float OutPositionSimd, OutNormalSimd;
		const bool Result = GJKRaycast2ImplSimd<VectorRegister4Float>(A, B, Rotation, Translation, RayDirSimd, static_cast<FRealSingle>(RayLength), OutTimeFloat, OutPositionSimd, OutNormalSimd, bComputeMTD, InitialDirSimd);

		OutTime = static_cast<double>(OutTimeFloat);

		alignas(16) FRealSingle OutFloat[4];
		VectorStoreAligned(OutNormalSimd, OutFloat);
		OutNormal.X = OutFloat[0];
		OutNormal.Y = OutFloat[1];
		OutNormal.Z = OutFloat[2];

		VectorStoreAligned(OutPositionSimd, OutFloat);
		OutPosition.X = OutFloat[0];
		OutPosition.Y = OutFloat[1];
		OutPosition.Z = OutFloat[2];

		return Result;
	}

	template <typename T = FReal, typename TGeometryA, typename TGeometryB>
	bool GJKRaycast2(const TGeometryA& A, const TGeometryB& B, const TRigidTransform<T, 3>& StartTM, const TVector<T, 3>& RayDir, const T RayLength,
		T& OutTime, TVector<T, 3>& OutPosition, TVector<T, 3>& OutNormal, const T GivenThicknessA = 0, bool bComputeMTD = false, const TVector<T, 3>& InitialDir = TVector<T, 3>(1, 0, 0), const T GivenThicknessB = 0)
	{
		return GJKRaycast2Impl(A, B, StartTM, RayDir, RayLength, OutTime, OutPosition, OutNormal, GivenThicknessA, bComputeMTD, InitialDir, GivenThicknessB);
	}

	inline bool GJKRaycast2SameSpaceImpl(const FGeomGJKHelperSIMD& A, const FGeomGJKHelperSIMD& B, const TVector<FReal, 3>& Start, const TVector<FReal, 3>& RayDir, const FReal RayLength,
		FReal& OutTime, TVector<FReal, 3>& OutPosition, TVector<FReal, 3>& OutNormal, const FReal GivenThicknessA, bool bComputeMTD, const TVector<FReal, 3>& InitialDir, const FReal GivenThicknessB)
	{
		const VectorRegister4Float UnusedRotation = GlobalVectorConstants::Float0001;
		const VectorRegister4Float Translation = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(Start.X, Start.Y, Start.Z, 0.0));

		const VectorRegister4Float InitialDirSimd = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(InitialDir[0], InitialDir[1], InitialDir[2], 0.0));
		const VectorRegister4Float RayDirSimd = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(RayDir[0], RayDir[1], RayDir[2], 0.0));

		FRealSingle OutTimeFloat = 0.0f;
		VectorRegister4Float OutPositionSimd, OutNormalSimd;
		const bool Result = GJKRaycast2ImplSimd<VectorRegister4Float, EGJKTestSpace::SameSpace>(A, B, UnusedRotation, Translation, RayDirSimd, static_cast<FRealSingle>(RayLength), OutTimeFloat, OutPositionSimd, OutNormalSimd, bComputeMTD, InitialDirSimd);

		OutTime = static_cast<double>(OutTimeFloat);

		alignas(16) FRealSingle OutFloat[4];
		VectorStoreAligned(OutNormalSimd, OutFloat);
		OutNormal.X = OutFloat[0];
		OutNormal.Y = OutFloat[1];
		OutNormal.Z = OutFloat[2];

		VectorStoreAligned(OutPositionSimd, OutFloat);
		OutPosition.X = OutFloat[0];
		OutPosition.Y = OutFloat[1];
		OutPosition.Z = OutFloat[2];

		return Result;
	}

	template <typename T = FReal, typename TGeometryA, typename TGeometryB>
	bool GJKRaycast2SameSpace(const TGeometryA& A, const TGeometryB& B, const TVector<T, 3>& Start, const TVector<T, 3>& RayDir, const T RayLength,
		T& OutTime, TVector<T, 3>& OutPosition, TVector<T, 3>& OutNormal, const T GivenThicknessA = 0, bool bComputeMTD = false, const TVector<T, 3>& InitialDir = TVector<T, 3>(1, 0, 0), const T GivenThicknessB = 0)
	{
		return GJKRaycast2SameSpaceImpl(A, B, Start, RayDir, RayLength, OutTime, OutPosition, OutNormal, GivenThicknessA, bComputeMTD, InitialDir, GivenThicknessB);
	}

	template <typename T, typename TGeometryA, typename TGeometryB>
	UE_DEPRECATED(5.5, "Use GJKDistanceInitialVFromDirection if possible, or GJKDistanceInitialVFromRelativeTransform if original behaviour was required")
	TVector<T, 3> GJKDistanceInitialV(const TGeometryA& A, const TGeometryB& B, const TRigidTransform<T, 3>& BToATM)
	{
		return GJKDistanceInitialVFromRelativeTransform(A, B, BToATM);
	}

	// Avoid this function - use GJKDistanceInitialVFromDirection that takes a vector direction and assumes the geometries are in the 
	// same space. If your geometries are not in the same space, they can be wrapped in TGJKShapeTransformed or TGJKCoreShapeTransformed
	// and you would already have these available because you need them for GJKDistance
	template <typename T, typename TGeometryA, typename TGeometryB>
	TVector<T, 3> GJKDistanceInitialVFromRelativeTransform(const TGeometryA& A, const TGeometryB& B, const TRigidTransform<T, 3>& BToATM)
	{
		FVec3 Direction = BToATM.GetTranslation();
		if (Direction.IsZero())
		{
			Direction = TVec3<T>(1, 0, 0);
		}
		const TVector<T, 3> DirectionInB = BToATM.GetRotation().Inverse() * Direction;

		int32 UnusedVertexIndex = INDEX_NONE;
		const TVector<T, 3> SupportA = A.SupportCore(Direction, A.GetMarginf(), (T*)nullptr, UnusedVertexIndex);
		const TVector<T, 3> SupportBLocal = B.SupportCore(-DirectionInB, B.GetMarginf(), (T*)nullptr, UnusedVertexIndex);
		const TVector<T, 3> SupportB = BToATM.TransformPositionNoScale(SupportBLocal);
		return SupportA - SupportB;
	}

	/*
	 * Can be used to generate the initial support direction for use with GJKDistance. Returns a point on the Minkowski Sum
	 * surface opposite to the direction of the supplied Direction. This is usually a good guess for the initial direction
	 * but it calls SupportCore on both shapes which in O(N) in the number of vertices, and there are often faster alternatives 
	 * if you know the type of shapes you are dealing with (e.g., return the vectors between the centers of the two convex shapes).
	 *
	 * If you do roll your own function, make sure that the vector returned is in or on the Minkowski sum (and don't just use a unit
	 * vector along some direction for example) or GJKDistance may early-exit with an inaccurate result.
	 */
	template <typename T, typename TGeometryA, typename TGeometryB>
	TVec3<T> GJKDistanceInitialVFromDirection(const TGeometryA& A, const TGeometryB& B, TVec3<T> Direction)
	{
		if (Direction.IsZero())
		{
			Direction = TVec3<T>(1, 0, 0);
		}
		int32 UnusedVertexIndex = INDEX_NONE;
		const TVec3<T> SupportA = A.SupportCore(Direction, A.GetMargin(), nullptr, UnusedVertexIndex);
		const TVec3<T> SupportB = B.SupportCore(-Direction, B.GetMargin(), nullptr, UnusedVertexIndex);
		return SupportA - SupportB;
	}

	// Status of a call to GJKDistance
	enum class EGJKDistanceResult
	{
		// The shapes are separated by a positive amount and all outputs have valid values
		Separated,

		// The shapes are overlapping by less than the net margin and all outputs have valid values (with a negative separation)
		Contact,

		// The shapes are overlapping by more than the net margin and all outputs are invalid (unchanged from their input values)
		DeepContact,
	};

	/**
	 * Find the distance and nearest points on two convex geometries A and B, if they do not overlap.
	 *
	 * This is intended to be used with TGJKShape, TGJKCoreShape, TGJKShapeTransformed and TGJKCoreShapeTransformed
	 * and generally you will have a TGJKShape or TGJKCoreShape as A, and a Transformed version for B. The transform in 
	 * B should transform a vector in B space to a vector in A space as all calculations are performed in the local-
	 * space of object A.
	 *
	 * You can also use this function when you have 2 shapes already in the same space to avoid transforms in the support
	 * function(s). In this case you would use the non-transform versions of TGJKShape and would need to transform the 
	 * results from the shared space to the space you desire.
	 *
	 * For algorithm see "A Fast and Robust GJK Implementation for Collision Detection of Convex Objects", Gino Van Deb Bergen, 1999.
	 * @note This algorithm aborts if objects are overlapping and it does not initialize the out parameters.
	 *
	 * @param A The first object (usually TGJKShape or TGJKCoreShape)
	 * @param B The second object (usually TGJKShapeTransformed or TGJKCoreShapeTransformed)
	 * @param InitialV  Starting support direction that must be in or on the Minkowski Sum. See GJKDistanceInitialVFromDirection().
	 * @param OutDistance If result is not DeepContact, the minimum distance between A and B, otherwise not modified.
	 * @param OutNearestA If result is not DeepContact, the near point on A in local-space, otherwise not modified.
	 * @param OutNearestB If result is not DeepContact, the near point on B in local-space, otherwise not modified.
	 * @param Epsilon The algorithm terminates when the iterative distance reduction gets below this threshold.
	 * @param MaxIts A limit on the number of iterations. Results may be approximate if this is too low.
	 * @return EGJKDistanceResult - see comments on the enum
	 */
	template <typename T, typename GJKShapeTypeA, typename GJKShapeTypeB>
	EGJKDistanceResult GJKDistance(const GJKShapeTypeA& A, const GJKShapeTypeB& B, const TVec3<T>& InitialV, T& OutDistance, TVec3<T>& OutNearestA, TVec3<T>& OutNearestB, TVec3<T>& OutNormalA, const T Epsilon = (T)1e-3, const int32 MaxIts = 16)
	{
		FSimplex SimplexIDs;
		TVec3<T> Simplex[4], SimplexA[4], SimplexB[4];
		T Barycentric[4] = { -1, -1, -1, -1 };

		const T AMargin = A.GetMargin();
		const T BMargin = B.GetMargin();
		T Mu = 0;

		// Initial vector in Minkowski(A - B)
		// NOTE: If InitialV is not in the Minkowski Sum we may incorrectly early-out in the bCloseEnough chaeck below
		TVec3<T> V = InitialV;
		T VLen = V.Size();
		int32 VertexIndexA = INDEX_NONE, VertexIndexB = INDEX_NONE;

		int32 It = 0;
		while (VLen > Epsilon)
		{
			// Find a new point in A-B that is closer to the origin
			// NOTE: we do not use support thickness here. Thickness is used when separating objects
			// so that GJK can find a solution, but that can be added in a later step.
			const TVec3<T> SupportA = A.SupportCore(-V, AMargin, nullptr, VertexIndexA);
			const TVec3<T> SupportB = B.SupportCore(V, BMargin, nullptr, VertexIndexB);
			const TVec3<T> W = SupportA - SupportB;

			T D = TVec3<T>::DotProduct(V, W) / VLen;
			Mu = FMath::Max(Mu, D);

			// See if we are still making progress toward the origin
			bool bCloseEnough = ((VLen - Mu) < Epsilon);
			if (bCloseEnough || (++It > MaxIts))
			{
				// We have reached the minimum to within tolerance. Or we have reached max iterations, in which
				// case we (probably) have a solution but with an error larger than Epsilon (technically we could be missing
				// the fact that we were going to eventually find the origin, but it'll be a close call so the approximation
				// is still good enough).
				if (SimplexIDs.NumVerts == 0)
				{
					// Our initial guess of V was already the minimum separating vector
					OutNearestA = SupportA;
					OutNearestB = SupportB;
				}
				else
				{
					// The simplex vertices are the nearest point/line/face
					OutNearestA = TVec3<T>(0, 0, 0);
					OutNearestB = TVec3<T>(0, 0, 0);
					for (int32 VertIndex = 0; VertIndex < SimplexIDs.NumVerts; ++VertIndex)
					{
						int32 WIndex = SimplexIDs[VertIndex];
						check(Barycentric[WIndex] >= (T)0);
						OutNearestA += Barycentric[WIndex] * SimplexA[WIndex];
						OutNearestB += Barycentric[WIndex] * SimplexB[WIndex];
					}
				}
				const TVec3<T> NormalA = -V / VLen;
				OutDistance = VLen - (AMargin + BMargin);
				OutNearestA += AMargin * NormalA;
				OutNearestB -= BMargin * NormalA;
				OutNormalA = NormalA;

				// NearestB should be in B-space but is currently in A-space
				OutNearestB = B.InverseTransformPositionNoScale(OutNearestB);

				return (OutDistance >= 0.0f) ? EGJKDistanceResult::Separated : EGJKDistanceResult::Contact;
			}

			// Add the new vertex to the simplex
			SimplexIDs[SimplexIDs.NumVerts] = SimplexIDs.NumVerts;
			Simplex[SimplexIDs.NumVerts] = W;
			SimplexA[SimplexIDs.NumVerts] = SupportA;
			SimplexB[SimplexIDs.NumVerts] = SupportB;
			++SimplexIDs.NumVerts;

			// Find the closest point to the origin on the simplex, and update the simplex to eliminate unnecessary vertices
			V = SimplexFindClosestToOrigin(Simplex, SimplexIDs, Barycentric, SimplexA, SimplexB);
			VLen = V.Size();
		}

		// Our geometries overlap - we did not set any outputs
		return EGJKDistanceResult::DeepContact;
	}

}