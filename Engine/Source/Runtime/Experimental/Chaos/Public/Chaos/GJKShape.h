// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"

namespace Chaos
{
	/**
	* Helpers and Wrappers for use with GJK to select the appropriate margin and support function
	* based on context. A different margin is used for sweeps and collisions,
	* and margins are used or not depending on the shape pair type involved.
	*/

	// Wraps an shape object (could be FImplicitObject or some other type with the required API)
	// and provides the API required for GJK, treating the shape as if it has zero margin. 
	// This means spheres will be spheres (not points with padding), convexes will be the outer hull, etc.
	//
	// See also TGJKShapeTransformed, TGJKCoreShape
	//
	// E.g., to use GJK between two Convex implicit objects
	//		GJKDistance(TGJKShape(ConvexA), TGJKShapeTransformed(ConvexB, BToATransform), ...);
	//
	// E.g., to use GJK between a convex and a sphere treated as a point with padding
	//		GJKDistance(TGJKShape(ConvexA), TGJKCoreShapeTransformed(SphereB, BToATransform), ...);
	//
	template<typename T_SHAPE>
	struct TGJKShape
	{
		using FShapeType = T_SHAPE;

		TGJKShape(const FShapeType& InShape) : Shape(InShape) {}

		FVec3 InverseTransformPositionNoScale(const FVec3& V) const
		{
			return V;
		}

		FReal GetMargin() const
		{
			return 0.0;
		}

		FRealSingle GetMarginf() const
		{
			return 0.0f;
		}

		FVec3 SupportCore(const FVec3 Dir, const FReal InMargin, FReal* OutSupportDelta, int32& VertexIndex) const
		{
			return Shape.Support(Dir, InMargin, VertexIndex);
		}

		bool IsConvex() const
		{
			return Shape.IsConvex();
		}

		FString ToString() const
		{
			return FString::Printf(TEXT("TGJKShape: %s"), *Shape.ToString());
		}

		const FShapeType& Shape;
	};

	// Like TGJKShape but for the second shape if it has a transform relative to the first.
	// 
	// @param InTransform The shape transform (relative to the first shape when used with GJK)
	// 
	// See also TGJKShape, TGJKCoreShape
	//
	template<typename T_SHAPE>
	struct TGJKShapeTransformed
	{
		using FShapeType = T_SHAPE;

		TGJKShapeTransformed(const FShapeType& InShape, const FRigidTransform3& InTransform) 
			: Transform(InTransform)
			, Shape(InShape)
		{}

		const FRigidTransform3& GetTransform() const
		{
			return Transform;
		}

		FVec3 InverseTransformPositionNoScale(const FVec3& V) const
		{
			return Transform.InverseTransformPositionNoScale(V);
		}

		FReal GetMargin() const
		{
			return 0.0f;
		}

		FVec3 SupportCore(const FVec3 Dir, const FReal InMargin, FReal* OutSupportDelta, int32& VertexIndex) const
		{
			const FVec3 LocalDir = Transform.InverseTransformVectorNoScale(Dir);
			const FVec3 LocalSupport = Shape.Support(LocalDir, InMargin, VertexIndex);
			return Transform.TransformPositionNoScale(LocalSupport);
		}

		bool IsConvex() const
		{
			return Shape.IsConvex();
		}

		FString ToString() const
		{
			return FString::Printf(TEXT("TGJKShapeTransformed: %s"), *Shape.ToString());
		}

		FRigidTransform3 Transform;
		const FShapeType& Shape;
	};



	// Like TGJKShape, but treats the shape as if it has a reduced "core" shape with a 
	// margin suitable for collision detection where significant overlaps are likely.
	// This means spheres will be points, convexes will be rounded shrunken hulls, etc.
	//
	// See also TGJKShape
	//
	// E.g., to use GJK of a sphere as a point against a marginless convex:
	//		GJKDistance(TGJKCoreShape(MySphere), TGJKShape(MyConvex), ...);
	//
	template<typename T_SHAPE>
	struct TGJKCoreShape
	{
		using FShapeType = T_SHAPE;

		TGJKCoreShape(const FShapeType& InShape)
			: Shape(InShape)
			, Margin(InShape.GetMarginf())
		{}

		TGJKCoreShape(const FShapeType& InShape, const FReal InMargin)
			: Shape(InShape)
			, Margin(FRealSingle(InMargin))
		{}

		TGJKCoreShape(const FShapeType& InShape, const FRealSingle InMargin)
			: Shape(InShape)
			, Margin(InMargin)
		{}

		FVec3 InverseTransformPositionNoScale(const FVec3& V) const
		{
			return V;
		}

		FReal GetMargin() const
		{
			return Margin;
		}

		FRealSingle GetMarginf() const
		{
			return Margin;
		}

		FVec3 SupportCore(const FVec3 Dir, const FReal InMargin, FReal* OutSupportDelta, int32& VertexIndex) const
		{
			return Shape.SupportCore(Dir, InMargin, OutSupportDelta, VertexIndex);
		}

		bool IsConvex() const
		{
			return Shape.IsConvex();
		}

		FString ToString() const
		{
			return FString::Printf(TEXT("TGJKCoreShape: %s, Margin: %f"), *Shape.ToString(), GetMargin());
		}

		const FShapeType& Shape;
		const FRealSingle Margin;
	};

	// Like TGJKCoreShape but for the second shape if it has a transform relative to the first.
	// 
	// @param InTransform The shape transform (relative to the first shape when used with GJK)
	// 
	// See also TGJKShapeTransformed, TGJKCoreShape
	//
	template<typename T_SHAPE>
	struct TGJKCoreShapeTransformed
	{
		using FShapeType = T_SHAPE;

		TGJKCoreShapeTransformed(const FShapeType& InShape, const FRigidTransform3& InTransform)
			: Transform(InTransform)
			, Shape(InShape)
			, Margin(InShape.GetMarginf())
		{}

		TGJKCoreShapeTransformed(const FShapeType& InShape, const FReal InMargin, const FRigidTransform3& InTransform)
			: Transform(InTransform)
			, Shape(InShape)
			, Margin(FRealSingle(InMargin))
		{}

		const FRigidTransform3& GetTransform() const
		{
			return Transform;
		}

		FVec3 InverseTransformPositionNoScale(const FVec3& V) const
		{
			return Transform.InverseTransformPositionNoScale(V);
		}

		FReal GetMargin() const
		{
			return FReal(Margin);
		}

		FRealSingle GetMarginf() const
		{
			return Margin;
		}

		FVec3 SupportCore(const FVec3 Dir, const FReal InMargin, FReal* OutSupportDelta, int32& VertexIndex) const
		{
			const FVec3 LocalDir = Transform.InverseTransformVectorNoScale(Dir);
			const FVec3 LocalSupport = Shape.SupportCore(LocalDir, InMargin, OutSupportDelta, VertexIndex);
			return Transform.TransformPositionNoScale(LocalSupport);
		}

		bool IsConvex() const
		{
			return Shape.IsConvex();
		}

		FString ToString() const
		{
			return FString::Printf(TEXT("TGJKCoreShapeTransformed: %s"), *Shape.ToString());
		}

		FRigidTransform3 Transform;
		const FShapeType& Shape;
		const FRealSingle Margin;
	};

	/**
	 * A sphere with minimal API for use in GJK/EPA.
	 * Equivalent to TGJKCoreShape<FImplicitSphere3> but avoids calls to the implicit
	*/
	class FGJKSphere
	{
	public:
		FGJKSphere(const FVec3 InCenter, const FReal InRadius)
			: Center(InCenter), Radius(InRadius)
		{
		}

		inline const FVec3& GetCenter() const
		{
			return Center;
		}

		inline FReal GetRadius() const
		{
			return Radius;
		}

		inline FVec3 InverseTransformPositionNoScale(const FVec3& V) const
		{
			return V;
		}

		inline const FVec3& SupportCore(const FVec3& Direction, const FReal InMargin, FReal* MaxMarginDelta, int32& VertexIndex) const
		{
			VertexIndex = 0;
			return Center;
		}

		inline FReal GetMargin() const
		{
			return Radius;
		}

		FString ToString() const
		{
			return FString::Printf(TEXT("FGJKSphere: Center: [%f, %f, %f], Radius: %f"), Center.X, Center.Y, Center.Z, Radius);
		}

	private:
		FVec3 Center;
		FReal Radius;
	};

	class FGJKSphereSingleSIMD
	{
	public:
		FGJKSphereSingleSIMD(const FVec3f& InCenter, FRealSingle InRadius)
			: Radius(InRadius)
		{
			Center = MakeVectorRegisterFloat(InCenter.X, InCenter.Y, InCenter.Z, 0.0f);
		}

		VectorRegister4Float SupportCoreSimd(const VectorRegister4Float V, const FRealSingle InMargin) const
		{
			return Center;
		}

		FRealSingle GetRadius() const
		{
			return Radius;
		}

		FRealSingle GetRadiusf() const
		{
			return Radius;
		}

		FRealSingle GetMargin() const
		{
			return Radius;
		}

		FRealSingle GetMarginf() const
		{
			return Radius;
		}

	private:
		VectorRegister4Float Center;
		FRealSingle Radius;
	};

	class FGJKCapsuleSingleSIMD
	{
	public:
		FGJKCapsuleSingleSIMD(const FVec3f& InPointA, const FVec3f& InPointB, const FVec3f& InAxis, const FRealSingle InRadius)
			: Radius(InRadius)
		{
			Axis = MakeVectorRegisterFloat(InAxis.X, InAxis.Y, InAxis.Z, 0.0f);
			PointA = MakeVectorRegisterFloat(InPointA.X, InPointA.Y, InPointA.Z, 0.0f);
			PointB = MakeVectorRegisterFloat(InPointB.X, InPointB.Y, InPointB.Z, 0.0f);
		}

		VectorRegister4Float SupportCoreSimd(const VectorRegister4Float V, FRealSingle InMargin) const
		{
			const VectorRegister4Float Dot = VectorDot3(V, Axis);
			// const bool bIsSecond = Dot >= 0;
			const VectorRegister4Float Mask = VectorCompareGE(Dot, GlobalVectorConstants::FloatZero);
			return VectorSelect(Mask, PointB, PointA);
		}

		FRealSingle GetRadius() const
		{
			return Radius;
		}

		FRealSingle GetRadiusf() const
		{
			return Radius;
		}

		FRealSingle GetMargin() const
		{
			return Radius;
		}

		FRealSingle GetMarginf() const
		{
			return Radius;
		}

	private:
		VectorRegister4Float Axis;
		VectorRegister4Float PointA;
		VectorRegister4Float PointB;
		FRealSingle Radius;
	};

	// Utility for creating TGJKShape objects using template parameter deduction
	template<typename T_SHAPE>
	TGJKShape<T_SHAPE> MakeGJKShape(const T_SHAPE& InShape)
	{
		return TGJKShape<T_SHAPE>(InShape);
	}

	// Utility for creating TGJKCoreShape objects using template parameter deduction
	template<typename T_SHAPE>
	TGJKCoreShape<T_SHAPE> MakeGJKCoreShape(const T_SHAPE& InShape)
	{
		return TGJKCoreShape<T_SHAPE>(InShape);
	}
}
