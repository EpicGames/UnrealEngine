// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/SimpleGeometryParticles.h"
#include "Chaos/ParticleHandleFwd.h"
#include "Chaos/GeometryParticlesfwd.h"
#include "Chaos/CollisionFilterData.h"
#include "Chaos/Collision/ParticleCollisions.h"
#include "Chaos/Box.h"
#include "Chaos/PhysicalMaterials.h"
#include "UObject/PhysicsObjectVersion.h"
#include "UObject/ExternalPhysicsCustomObjectVersion.h"
#include "UObject/FortniteValkyrieBranchObjectVersion.h"
#include "UObject/ExternalPhysicsMaterialCustomObjectVersion.h"
#include "Chaos/Properties.h"
#include "Chaos/Framework/PhysicsProxyBase.h"
#include "Chaos/Framework/PhysicsSolverBase.h"
#include "Chaos/ShapeInstance.h"

#ifndef CHAOS_DETERMINISTIC
#define CHAOS_DETERMINISTIC 1
#endif


namespace Chaos
{
	class FConstraintHandle;
	class FParticleCollisions;

	using FConstraintHandleArray = TArray<FConstraintHandle*>;

	namespace CVars
	{
		CHAOS_API extern int32 CCDAxisThresholdMode;
		CHAOS_API extern bool bCCDAxisThresholdUsesProbeShapes;
	}

	namespace Private
	{
		class FPBDIslandParticle;

		CHAOS_API extern FString EmptyParticleName;
	}

	/**
	* Union between shape and shapes array pointers, used for passing around shapes with
	* implicit that could be single implicit or union.
	*/
	class FShapeOrShapesArray
	{
	public:

		// Store particle's shape array if particle has union geometry, otherwise individual shape.
		FShapeOrShapesArray(const FGeometryParticleHandle* Particle);

		FShapeOrShapesArray()
			: Shape(nullptr)
			, bIsSingleShape(true)
		{}

		FShapeOrShapesArray(const FPerShapeData* InShape)
			: Shape(InShape)
			, bIsSingleShape(true)
		{}

		FShapeOrShapesArray(const FShapesArray* InShapeArray)
			: ShapeArray(InShapeArray)
			, bIsSingleShape(false)
		{}

		bool IsSingleShape() const { return bIsSingleShape; }

		bool IsValid() const { return Shape != nullptr; }

		// Do not call without checking IsSingleShape().
		const FPerShapeData* GetShape() const
		{
			check(bIsSingleShape);
			return Shape;
		}

		// Do not call without checking IsSingleShape().
		const FShapesArray* GetShapesArray() const
		{
			check(!bIsSingleShape);
			return ShapeArray;
		}

	private:
		union
		{
			const FPerShapeData* Shape;
			const FShapesArray* ShapeArray;
		};

		bool bIsSingleShape;
	};

	FORCEINLINE uint32 GetTypeHash(const FParticleID& Unique)
	{
		return ::GetTypeHash(Unique.GlobalID);
	}

	//Holds the data for getting back at the real handle if it's still valid
	//Systems should not use this unless clean-up of direct handle is slow, this uses thread safe shared ptr which is not cheap
	class FWeakParticleHandle
	{
	public:

		FWeakParticleHandle() = default;
		FWeakParticleHandle(TGeometryParticleHandle<FReal,3>* InHandle) : SharedData(MakeShared<FData, ESPMode::ThreadSafe>(FData{InHandle})){}

		//Assumes the weak particle handle has been initialized so SharedData must exist
		TGeometryParticleHandle<FReal,3>* GetHandleUnsafe() const
		{
			return SharedData->Handle;
		}

		TGeometryParticleHandle<FReal,3>* GetHandle() const { return SharedData ? SharedData->Handle : nullptr; }
		void ResetHandle()
		{
			if(SharedData)
			{
				SharedData->Handle = nullptr;
			}
		}

		bool IsInitialized()
		{
			return SharedData != nullptr;
		}

	private:

		struct FData
		{
			TGeometryParticleHandle<FReal,3>* Handle;
		};
		TSharedPtr<FData,ESPMode::ThreadSafe> SharedData;
	};

	enum class EResimType : uint8
	{
		FullResim = 0,	//fully re-run simulation and keep results (any forces must be applied again)
		ResimAsFollower = 1, //use previous forces and snap to previous results regardless of variation - used to push other objects away
		FrozenDuringResim = 2, // this particle is "frozen" (made kinematic), updates and integration should be skipped
		//ResimWithPrevForces, //use previous forces and keep results (UNIMPLEMENTED)
	};
	
	template<class T, int d, EGeometryParticlesSimType SimType>
	class TGeometryParticlesImp : public TSimpleGeometryParticles<T, d>
	{
	public:

		using TArrayCollection::Size;
		using TParticles<T,d>::GetX;
		using TSimpleGeometryParticles<T, d>::GetR;
		using TSimpleGeometryParticles<T, d>::GetGeometry;
		using TSimpleGeometryParticles<T, d>::SetGeometry;
		using TSimpleGeometryParticles<T, d>::GetAllGeometry;

		CHAOS_API static TGeometryParticlesImp<T, d, SimType>* SerializationFactory(FChaosArchive& Ar, TGeometryParticlesImp < T, d, SimType>* Particles);
		
		TGeometryParticlesImp()
		    : TSimpleGeometryParticles<T, d>()
			, MContainerListMask(EGeometryParticleListMask::None)
		{
			MParticleType = EParticleType::Static;
			RegisterArrays();
		}
		TGeometryParticlesImp(const TGeometryParticlesImp<T, d, SimType>& Other) = delete;
		TGeometryParticlesImp(TGeometryParticlesImp<T, d, SimType>&& Other)
		    : TSimpleGeometryParticles<T, d>(MoveTemp(Other))
			, MContainerListMask(Other.MContainerListMask)
			, MUniqueIdx(MoveTemp(Other.MUniqueIdx))
			, MGeometryParticleHandle(MoveTemp(Other.MGeometryParticleHandle))
			, MGeometryParticle(MoveTemp(Other.MGeometryParticle))
			, MPhysicsProxy(MoveTemp(Other.MPhysicsProxy))
			, MHasCollision(MoveTemp(Other.MHasCollision))
			, MShapesArray(MoveTemp(Other.MShapesArray))
			, MLocalBounds(MoveTemp(Other.MLocalBounds))
			, MCCDAxisThreshold(MoveTemp(Other.MCCDAxisThreshold))
			, MWorldSpaceInflatedBounds(MoveTemp(Other.MWorldSpaceInflatedBounds))
			, MHasBounds(MoveTemp(Other.MHasBounds))
			, MSpatialIdx(MoveTemp(Other.MSpatialIdx))
			, MSyncState(MoveTemp(Other.MSyncState))
			, MWeakParticleHandle(MoveTemp(Other.MWeakParticleHandle))
			, MParticleConstraints(MoveTemp(Other.MParticleConstraints))
			, MParticleCollisions(MoveTemp(Other.MParticleCollisions))
			, MGraphNode(MoveTemp(Other.MGraphNode))
			, MResimType(MoveTemp(Other.MResimType))
			, MEnabledDuringResim(MoveTemp(Other.MEnabledDuringResim))
			, MLightWeightDisabled(MoveTemp(Other.MLightWeightDisabled))
#if CHAOS_DETERMINISTIC
			, MParticleIDs(MoveTemp(Other.MParticleIDs))
#endif
#if CHAOS_DEBUG_NAME
			, MDebugName(MoveTemp(Other.MDebugName))
#endif

		{
			MParticleType = EParticleType::Static;
			RegisterArrays();
		}

		static constexpr bool IsRigidBodySim() { return SimType == EGeometryParticlesSimType::RigidBodySim; }

		TGeometryParticlesImp(TParticles<T, d>&& Other)
		    : TSimpleGeometryParticles<T, d>(MoveTemp(Other))
			, MContainerListMask(EGeometryParticleListMask::None)
		{
			MParticleType = EParticleType::Static;
			RegisterArrays();
		}

		virtual ~TGeometryParticlesImp() = default;

		FUniqueIdx UniqueIdx(const int32 Index) const { return MUniqueIdx[Index]; }
		FUniqueIdx& UniqueIdx(const int32 Index) { return MUniqueIdx[Index]; }

		ESyncState& SyncState(const int32 Index) { return MSyncState[Index].State; }
		ESyncState SyncState(const int32 Index) const { return MSyncState[Index].State; }


		UE_DEPRECATED(5.4, "Please use GetGeometry instead")
		TSerializablePtr<FImplicitObject> Geometry(const int32 Index) const { check(false); return TSerializablePtr<FImplicitObject>(); }
		
		UE_DEPRECATED(5.4, "Please use GetGeometry instead")
		const TUniquePtr<FImplicitObject>& DynamicGeometry(const int32 Index) const
		{
			check(false);
			static TUniquePtr<FImplicitObject> DummyPtr;
			return DummyPtr;
		}
		
		UE_DEPRECATED(5.4, "Please use GetGeometry instead")
		const TSharedPtr<const FImplicitObject, ESPMode::ThreadSafe>& SharedGeometry(const int32 Index) const
		{
			check(false);
			static TSharedPtr<const FImplicitObject, ESPMode::ThreadSafe> DummyPtr;
			return DummyPtr;
		}

		bool HasCollision(const int32 Index) const { return MHasCollision[Index]; }
		bool& HasCollision(const int32 Index) { return MHasCollision[Index]; }

		const FShapesArray& ShapesArray(const int32 Index) const { return reinterpret_cast<const FShapesArray&>(MShapesArray[Index]); }
		void RemoveShapesAtSortedIndices(const int32 ParticleIndex, const TArrayView<const int32>& InIndices);

		const FShapeInstanceArray& ShapeInstances(const int32 Index) const { return MShapesArray[Index]; }

#if CHAOS_DETERMINISTIC
		FParticleID ParticleID(const int32 Idx) const { return MParticleIDs[Idx]; }
		FParticleID& ParticleID(const int32 Idx) { return MParticleIDs[Idx]; }
#endif
		
		UE_DEPRECATED(5.4, "Please use SetGeometry with FImplicitObjectPtr instead")
		void SetDynamicGeometry(const int32 Index, TUniquePtr<FImplicitObject>&& InUnique) { check(false); }

		UE_DEPRECATED(5.4, "Please use SetGeometry with FImplicitObjectPtr instead")
		void SetSharedGeometry(const int32 Index, TSharedPtr<const FImplicitObject, ESPMode::ThreadSafe> InShared) { check(false); }

	private:
		void RegisterArrays()
		{
			TArrayCollection::AddArray(&MUniqueIdx);
#if CHAOS_DETERMINISTIC
			TArrayCollection::AddArray(&MParticleIDs);
#endif
			TArrayCollection::AddArray(&MHasCollision);
			TArrayCollection::AddArray(&MShapesArray);
			TArrayCollection::AddArray(&MLocalBounds);
			TArrayCollection::AddArray(&MCCDAxisThreshold);
			TArrayCollection::AddArray(&MWorldSpaceInflatedBounds);
			TArrayCollection::AddArray(&MHasBounds);
			TArrayCollection::AddArray(&MSpatialIdx);
			TArrayCollection::AddArray(&MSyncState);
			TArrayCollection::AddArray(&MWeakParticleHandle);
			TArrayCollection::AddArray(&MParticleConstraints);
			TArrayCollection::AddArray(&MParticleCollisions);
			TArrayCollection::AddArray(&MGraphNode);
			TArrayCollection::AddArray(&MResimType);
			TArrayCollection::AddArray(&MEnabledDuringResim);
			TArrayCollection::AddArray(&MLightWeightDisabled);
			TArrayCollection::AddArray(&MParticleListMask);

#if CHAOS_DEBUG_NAME
			TArrayCollection::AddArray(&MDebugName);
#endif

			if (IsRigidBodySim())
			{
				TArrayCollection::AddArray(&MGeometryParticleHandle);
				TArrayCollection::AddArray(&MGeometryParticle);
				TArrayCollection::AddArray(&MPhysicsProxy);
			}
		}

		virtual void SetGeometryImpl(const int32 Index, const FImplicitObjectPtr& InGeometry) override
		{
			// We hit these checks if there's a call to modify geometry without first clearing constraints 
			// on the particle (e.g., PBDRigidsEvolutionGBF::InvalidateParticle)
			check(MParticleCollisions[Index].Num() == 0);
			//check(MParticleConstraints[Index].Num() == 0);

			TSimpleGeometryParticles<T, d>::SetGeometryImpl(Index, InGeometry);

			UpdateShapesArray(Index);

			MHasBounds[Index] = (InGeometry && InGeometry->HasBoundingBox());
			if (MHasBounds[Index])
			{
				MLocalBounds[Index] = TAABB<T, d>(InGeometry->BoundingBox());

				// Update the threshold we use to determine when to enable CCD. This is based on the bounds.
				UpdateCCDAxisThreshold(Index);

				// Update the world-space stat of all the shapes - must be called after UpdateShapesArray
				// world space inflated bounds needs to take expansion into account - this is done in integrate for dynamics anyway, so
				// this computation is mainly for statics
				UpdateWorldSpaceState(Index, TRigidTransform<FReal, 3>(GetX(Index), GetR(Index)), FVec3(0));
			}
		}

		void UpdateCCDAxisThreshold(const int32 Index)
		{
			// NOTE: We get empty bounds (as opposed to no bounds) if we have Geometry that is an empty Union
			if (!MHasBounds[Index] || MLocalBounds[Index].IsEmpty())
			{
				MCCDAxisThreshold[Index] = FVec3(0);
				return;
			}

			if (CVars::CCDAxisThresholdMode == 0)
			{
				// Use object extents as CCD axis threshold
				MCCDAxisThreshold[Index] = MLocalBounds[Index].Extents();
			}
			else if (CVars::CCDAxisThresholdMode == 1)
			{
				// Use thinnest object extents as all axis CCD thresholds
				MCCDAxisThreshold[Index] = FVec3(MLocalBounds[Index].Extents().GetMin());
			}
			else
			{
				// Find minimum shape bounds thickness on each axis
				FVec3 ThinnestBoundsPerAxis = MLocalBounds[Index].Extents();
				for (const TUniquePtr<FPerShapeData>& Shape : ShapesArray(Index))
				{
					// Only sim-enabled shapes should ever be swept with CCD, so make sure the
					// sim-enabled flag is on for each shape before considering it's min bounds
					// for CCD extents.
					if (Shape->GetSimEnabled() && (CVars::bCCDAxisThresholdUsesProbeShapes || !Shape->GetIsProbe()))
					{
						const FImplicitObjectRef Geometry = Shape->GetGeometry();
						if (Geometry && Geometry->HasBoundingBox())
						{
							const TVector<T, d> ShapeExtents = Geometry->BoundingBox().Extents();
							TVector<T, d>& CCDAxisThreshold = MCCDAxisThreshold[Index];
							for (int32 AxisIndex = 0; AxisIndex < d; ++AxisIndex)
							{
								ThinnestBoundsPerAxis[AxisIndex] = FMath::Min(ShapeExtents[AxisIndex], ThinnestBoundsPerAxis[AxisIndex]);
							}
						}
					}
				}

				if (CVars::CCDAxisThresholdMode == 2)
				{
					// On each axis, use the thinnest shape bound on that axis
					MCCDAxisThreshold[Index] = ThinnestBoundsPerAxis;
				}
				else if (CVars::CCDAxisThresholdMode == 3)
				{
					// Find the thinnest shape bound on any axis and use this for all axes
					MCCDAxisThreshold[Index] = FVec3(ThinnestBoundsPerAxis.GetMin());
				}
			}
		}
	public:

		const TAABB<T,d>& LocalBounds(const int32 Index) const
		{
			return MLocalBounds[Index];
		}

		TAABB<T, d>& LocalBounds(const int32 Index)
		{
			return MLocalBounds[Index];
		}

		const TVector<T,d>& CCDAxisThreshold(const int32 Index) const
		{
			return MCCDAxisThreshold[Index];
		}

		bool HasBounds(const int32 Index) const
		{
			return MHasBounds[Index];
		}

		bool& HasBounds(const int32 Index)
		{
			return MHasBounds[Index];
		}

		FSpatialAccelerationIdx SpatialIdx(const int32 Index) const
		{
			return MSpatialIdx[Index];
		}

		FSpatialAccelerationIdx& SpatialIdx(const int32 Index)
		{
			return MSpatialIdx[Index];
		}

#if CHAOS_DEBUG_NAME
		const TSharedPtr<FString, ESPMode::ThreadSafe>& DebugName(const int32 Index) const
		{
			return MDebugName[Index];
		}

		TSharedPtr<FString, ESPMode::ThreadSafe>& DebugName(const int32 Index)
		{
			return MDebugName[Index];
		}
#endif

		const FString& GetDebugName(const int32 Index) const
		{
#if CHAOS_DEBUG_NAME
			if (MDebugName[Index].IsValid())
			{
				return *(MDebugName[Index].Get());
			}
#endif
			return Private::EmptyParticleName;
		}

		const TAABB<T, d>& WorldSpaceInflatedBounds(const int32 Index) const
		{
			return MWorldSpaceInflatedBounds[Index];
		}

		void UpdateWorldSpaceState(const int32 Index, const FRigidTransform3& WorldTransform, const FVec3& BoundsExpansion)
		{
			const FShapesArray& Shapes = ShapesArray(Index);

			// If we have no shapes we are have a point bounds at our local origin
			FAABB3 WorldBounds = (Shapes.Num() > 0) ? FAABB3::EmptyAABB() : FAABB3::ZeroAABB();

			// NOTE: Individual shape bounds are not expanded. We only require that the particle bounds is expanded because that is used
			// in the broadphase. The midphase is what requires the shape bounds and it also has all the information required to expand the shape bounds as needed.
			for (const auto& Shape : Shapes)
			{
				Shape->UpdateWorldSpaceState(WorldTransform);
				WorldBounds.GrowToInclude(Shape->GetWorldSpaceShapeBounds());
			}

			MWorldSpaceInflatedBounds[Index] = TAABB<T, d>(WorldBounds).ThickenSymmetrically(BoundsExpansion);
			ensureMsgf(MWorldSpaceInflatedBounds[Index].IsValid(), TEXT("Invalid world space inflated bounds. %s, BoundsExpansion(%s)"),
				*MWorldSpaceInflatedBounds[Index].ToString(), *BoundsExpansion.ToString());
		}

		void UpdateWorldSpaceStateSwept(const int32 Index, const FRigidTransform3& WorldTransform, const FVec3& BoundsExpansion, const FVec3& DeltaX)
		{
			// Update the bounds of all shapes (individual shape bounds are not expanded) and accumulate the net bounds
			UpdateWorldSpaceState(Index, WorldTransform, BoundsExpansion);

			// Apply the swept bounds delta
			MWorldSpaceInflatedBounds[Index].GrowByVector(DeltaX);
			ensureMsgf(MWorldSpaceInflatedBounds[Index].IsValid(), TEXT("Invalid world space inflated bounds. %s, BoundsExpansion(%s), DeltaX(%s)"),
				*MWorldSpaceInflatedBounds[Index].ToString(), *BoundsExpansion.ToString(), *DeltaX.ToString());
		}

		typedef FGeometryParticleHandle THandleType;
		FORCEINLINE THandleType* Handle(int32 Index) const { return const_cast<THandleType*>(MGeometryParticleHandle[Index].Get()); }

		CHAOS_API void SetHandle(int32 Index, FGeometryParticleHandle* Handle);
		
		FGeometryParticle* GTGeometryParticle(const int32 Index) const { return MGeometryParticle[Index]; }
		FGeometryParticle*& GTGeometryParticle(const int32 Index) { return MGeometryParticle[Index]; }

		const IPhysicsProxyBase* PhysicsProxy(const int32 Index) const { return MPhysicsProxy[Index];  }
		IPhysicsProxyBase* PhysicsProxy(const int32 Index) { return MPhysicsProxy[Index]; }
		void SetPhysicsProxy(const int32 Index, IPhysicsProxyBase* InPhysicsProxy)
		{
			MPhysicsProxy[Index] = InPhysicsProxy;
		}

		FWeakParticleHandle& WeakParticleHandle(const int32 Index)
		{
			FWeakParticleHandle& WeakHandle = MWeakParticleHandle[Index];
			if(WeakHandle.IsInitialized())
			{
				return WeakHandle;
			}

			WeakHandle = FWeakParticleHandle(Handle(Index));
			return WeakHandle;
		}

		/**
		 * @brief All of the persistent (non-collision) constraints affecting the particle
		*/
		FConstraintHandleArray& ParticleConstraints(const int32 Index)
		{
			return MParticleConstraints[Index];
		}

		void AddConstraintHandle(const int32& Index, FConstraintHandle* InConstraintHandle)
		{
			CHAOS_ENSURE(!MParticleConstraints[Index].Contains(InConstraintHandle));
			MParticleConstraints[Index].Add(InConstraintHandle);
		}


		void RemoveConstraintHandle(const int32& Index, FConstraintHandle* InConstraintHandle)
		{
			MParticleConstraints[Index].RemoveSingleSwap(InConstraintHandle);
			CHAOS_ENSURE(!MParticleConstraints[Index].Contains(InConstraintHandle));
		}

		/**
		 * @brief All of the collision constraints affecting the particle
		*/
		FParticleCollisions& ParticleCollisions(const int32 Index)
		{
			return MParticleCollisions[Index];
		}

		FORCEINLINE Private::FPBDIslandParticle* ConstraintGraphNode(const int32 Index) const { return MGraphNode[Index]; }
		FORCEINLINE Private::FPBDIslandParticle*& ConstraintGraphNode(const int32 Index) { return MGraphNode[Index]; }

		FORCEINLINE EResimType ResimType(const int32 Index) const { return MResimType[Index]; }
		FORCEINLINE EResimType& ResimType(const int32 Index) { return MResimType[Index]; }

		FORCEINLINE bool EnabledDuringResim(const int32 Index) const { return MEnabledDuringResim[Index]; }
		FORCEINLINE bool& EnabledDuringResim(const int32 Index) { return MEnabledDuringResim[Index]; }

		FORCEINLINE bool LightWeightDisabled(const int32 Index) const { return MLightWeightDisabled[Index]; }
		FORCEINLINE bool& LightWeightDisabled(const int32 Index) { return MLightWeightDisabled[Index]; }

		FORCEINLINE EGeometryParticleListMask ListMask(const int32 Index) const { return MParticleListMask[Index]; }
		FORCEINLINE EGeometryParticleListMask& ListMask(const int32 Index) { return MParticleListMask[Index]; }

		// Deprecated API
		UE_DEPRECATED(5.3, "Use ConstraintGraphNode") const int32 ConstraintGraphIndex(const int32 Index) const { return INDEX_NONE; }
		UE_DEPRECATED(5.3, "Use ConstraintGraphNode") int32& ConstraintGraphIndex(const int32 Index) { static int32 Dummy = INDEX_NONE; return Dummy; }

private:
		friend THandleType;
		void ResetWeakParticleHandle(const int32 Index)
		{
			FWeakParticleHandle& WeakHandle = MWeakParticleHandle[Index];
			if(WeakHandle.IsInitialized())
			{
				return WeakHandle.ResetHandle();
			}
		}
public:

		FString ToString(int32 index) const
		{
			FString BaseString = TParticles<T, d>::ToString(index);
			return FString::Printf(TEXT("%s, MUniqueIdx:%d MR:%s, MGeometry:%s"), *BaseString, UniqueIdx(index).Idx, *GetR(index).ToString(), (GetGeometry(index) ? *(GetGeometry(index)->ToString()) : TEXT("none")));
		}

		virtual void Serialize(FChaosArchive& Ar) override
		{
			LLM_SCOPE(ELLMTag::ChaosParticles);
			TSimpleGeometryParticles<T, d>::Serialize(Ar);
			
			Ar.UsingCustomVersion(FPhysicsObjectVersion::GUID);
			if (Ar.CustomVer(FPhysicsObjectVersion::GUID) >= FPhysicsObjectVersion::PerShapeData)
			{
				Ar << MShapesArray;
			}

			if (Ar.CustomVer(FPhysicsObjectVersion::GUID) >= FPhysicsObjectVersion::SerializeGTGeometryParticles)
			{
				SerializeGeometryParticleHelper(Ar, this);
			}

			Ar.UsingCustomVersion(FExternalPhysicsCustomObjectVersion::GUID);
			if (Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) >= FExternalPhysicsCustomObjectVersion::SerializeParticleBounds)
			{
				TBox<FReal, 3>::SerializeAsAABBs(Ar, MLocalBounds);
				TBox<FReal, 3>::SerializeAsAABBs(Ar, MWorldSpaceInflatedBounds);
				Ar << MHasBounds;

				if (Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) < FExternalPhysicsCustomObjectVersion::SerializeShapeWorldSpaceBounds)
				{
					for (int32 Idx = 0; Idx < MShapesArray.Num(); ++Idx)
					{
						UpdateWorldSpaceState(Idx, FRigidTransform3(GetX(Idx), GetR(Idx)), FVec3(0));
					}
				}
			}
			else
			{
				//just assume all bounds come from geometry (technically wrong for pbd rigids with only sample points, but backwards compat is not that important right now)
				for (int32 Idx = 0; Idx < GetAllGeometry().Num(); ++Idx)
				{
					MHasBounds[Idx] = GetGeometry(Idx) && GetGeometry(Idx)->HasBoundingBox();
					if (MHasBounds[Idx])
					{
						MLocalBounds[Idx] = TAABB<T, d>(GetGeometry(Idx)->BoundingBox());
						//ignore velocity too, really just trying to get something reasonable)
						UpdateWorldSpaceState(Idx, FRigidTransform3(GetX(Idx), GetR(Idx)), FVec3(0));
					}
				}
			}

			if (Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) < FExternalPhysicsCustomObjectVersion::SpatialIdxSerialized)
			{
				MSpatialIdx.AddZeroed(GetAllGeometry().Num());
			}
			else
			{
				Ar << MSpatialIdx;
			}

			if (Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) < FExternalPhysicsCustomObjectVersion::SerializeHashResult)
			{
				//no longer care about hash so don't read it and don't do anything
			}
		}

		FORCEINLINE EParticleType ParticleType() const { return MParticleType; }

		FORCEINLINE EGeometryParticleListMask GetContainerListMask() const { return MContainerListMask; }
		void SetContainerListMask(const EGeometryParticleListMask InMask) { MContainerListMask = InMask; }

		FORCEINLINE TArray<TAABB<T, d>>& AllLocalBounds() { return MLocalBounds; }
		FORCEINLINE TArray<TAABB<T, d>>& AllWorldSpaceInflatedBounds() { return MWorldSpaceInflatedBounds; }
		FORCEINLINE TArray<bool>& AllHasBounds() { return MHasBounds; }

	protected:
		EParticleType MParticleType;
		EGeometryParticleListMask MContainerListMask;

	private:
		TArrayCollectionArray<FUniqueIdx> MUniqueIdx;
		TArrayCollectionArray<TSerializablePtr<FGeometryParticleHandle>> MGeometryParticleHandle;
		TArrayCollectionArray<FGeometryParticle*> MGeometryParticle;
		TArrayCollectionArray<IPhysicsProxyBase*> MPhysicsProxy;
		TArrayCollectionArray<bool> MHasCollision;
		TArrayCollectionArray<FShapeInstanceArray> MShapesArray;
		TArrayCollectionArray<TAABB<T,d>> MLocalBounds;
		TArrayCollectionArray<TVector<T,d>> MCCDAxisThreshold;
		TArrayCollectionArray<TAABB<T, d>> MWorldSpaceInflatedBounds;
		TArrayCollectionArray<bool> MHasBounds;
		TArrayCollectionArray<FSpatialAccelerationIdx> MSpatialIdx;
		TArrayCollectionArray<FSyncState> MSyncState;
		TArrayCollectionArray<FWeakParticleHandle> MWeakParticleHandle;
		TArrayCollectionArray<FConstraintHandleArray> MParticleConstraints;
		TArrayCollectionArray<FParticleCollisions> MParticleCollisions;
		TArrayCollectionArray<Private::FPBDIslandParticle*> MGraphNode;
		TArrayCollectionArray<EResimType> MResimType;
		TArrayCollectionArray<bool> MEnabledDuringResim;
		TArrayCollectionArray<bool> MLightWeightDisabled;
		TArrayCollectionArray<EGeometryParticleListMask> MParticleListMask;

		CHAOS_API void UpdateShapesArray(const int32 Index);

		template <typename T2, int d2, EGeometryParticlesSimType SimType2>
		friend class TGeometryParticlesImp;

		CHAOS_API void SerializeGeometryParticleHelper(FChaosArchive& Ar, TGeometryParticlesImp<T, d, EGeometryParticlesSimType::RigidBodySim>* GeometryParticles);
		
		void SerializeGeometryParticleHelper(FChaosArchive& Ar, TGeometryParticlesImp<T, d, EGeometryParticlesSimType::Other>* GeometryParticles)
		{
			check(false);	//cannot serialize this sim type
		}

#if CHAOS_DETERMINISTIC
		TArrayCollectionArray<FParticleID> MParticleIDs;
#endif
#if CHAOS_DEBUG_NAME
		TArrayCollectionArray<TSharedPtr<FString, ESPMode::ThreadSafe>> MDebugName;
#endif
	};

	template <typename T, int d, EGeometryParticlesSimType SimType>
	FChaosArchive& operator<<(FChaosArchive& Ar, TGeometryParticlesImp<T, d, SimType>& Particles)
	{
		Particles.Serialize(Ar);
		return Ar;
	}

	template <>
	void TGeometryParticlesImp<FReal, 3, EGeometryParticlesSimType::Other>::SetHandle(int32 Index, TGeometryParticleHandle<FReal, 3>* Handle);

	template<>
	TGeometryParticlesImp<FReal, 3, EGeometryParticlesSimType::Other>* TGeometryParticlesImp<FReal, 3, EGeometryParticlesSimType::Other>::SerializationFactory(FChaosArchive& Ar, TGeometryParticlesImp<FReal, 3, EGeometryParticlesSimType::Other>* Particles);

}

