// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Chaos/Real.h"

namespace Chaos
{
	//Used for down casting when iterating over multiple SOAs.
	enum class EParticleType : uint8
	{
		Static,
		Kinematic,
		Rigid,
		Clustered,	//only applicable on physics thread side
		StaticMesh,
		SkeletalMesh,
		GeometryCollection,
		Unknown
	};

	enum class EGeometryParticlesSimType
	{
		RigidBodySim,
		Other
	};

	enum class ESyncState: uint8
	{
		InSync,	    //in sync with recorded data
		//SoftDesync, //recorded data still matches, but may interact with hard desynced particles
		HardDesync, //recorded data mismatches, must run collision detection again
	};

	struct FSyncState
	{
		ESyncState State;

		FSyncState()
		: State(ESyncState::InSync)
		{

		}
	};

	template<class T, int d, EGeometryParticlesSimType SimType>
	class TGeometryParticlesImp;

	template <typename T, int d>
	using TGeometryParticles = TGeometryParticlesImp<T, d, EGeometryParticlesSimType::RigidBodySim>;

	using FGeometryParticles = TGeometryParticles<FReal, 3>;

	template <typename T, int d>
	using TGeometryClothParticles = TGeometryParticlesImp<T, d, EGeometryParticlesSimType::Other>;


	struct FSpatialAccelerationIdx
	{
		uint16 Bucket : 3;
		uint16 InnerIdx : 13;

		static constexpr uint16 MaxBucketEntries = 1 << 13;
		static constexpr uint16 MaxBuckets = 1 << 3;

		FSpatialAccelerationIdx() : Bucket(0), InnerIdx(0) {};
		FSpatialAccelerationIdx(uint16 inBucket, uint16 inInnerIdx) : Bucket(inBucket), InnerIdx(inInnerIdx) {};

		bool operator==(const FSpatialAccelerationIdx& Rhs) const
		{
			return ((const uint16&)*this) == ((const uint16&)Rhs);
		}
	};

	inline uint32 GetTypeHash(const FSpatialAccelerationIdx& Idx)
	{
		return ::GetTypeHash((const uint16&) Idx);
	}

	inline FArchive& operator<<(FArchive& Ar, FSpatialAccelerationIdx& Idx)
	{
		return Ar << (uint16&)Idx;
	}


	struct FUniqueIdx
	{
		int32 Idx;
		FUniqueIdx(): Idx(INDEX_NONE){}
		explicit FUniqueIdx(int32 InIdx): Idx(InIdx){}

		bool IsValid() const { return Idx != INDEX_NONE; }
		bool operator<(const FUniqueIdx& Other) const { return Idx < Other.Idx; }
		bool operator==(const FUniqueIdx& Other) const { return Idx == Other.Idx; }
	};

	FORCEINLINE uint32 GetTypeHash(const FUniqueIdx& Unique)
	{
		return ::GetTypeHash(Unique.Idx);
	}

	//Used for down casting when iterating over multiple SOAs.
	enum class EInternalClusterType : uint8
	{
		None,
		KinematicOrStatic,
		Dynamic,
		ClusterUnion
	};

	/**
	* Used to keep track of which Lists a particle is in
	*/
	enum class EGeometryParticleListMask
	{
		None								= 0,

		//
		// NOTE: The contents of this enum match the members of FPBDRigidsSOA
		//

		// Particle containers (each particle can only be in one of these by definition)
		StaticParticles						= 1 << 0,
		StaticDisabledParticles				= 1 << 1,
		KinematicParticles					= 1 << 2,
		KinematicDisabledParticles			= 1 << 3,
		DynamicDisabledParticles			= 1 << 4,
		DynamicParticles					= 1 << 5,
		DynamicKinematicParticles			= 1 << 6,
		ClusteredParticles					= 1 << 7,
		GeometryCollectionParticles			= 1 << 8,

		// Particle lists (each particle can be zero or more of these, depending on state
		StaticGeometryCollectionArray		= 1 << 9,
		KinematicGeometryCollectionArray	= 1 << 10,
		SleepingGeometryCollectionArray		= 1 << 11,
		DynamicGeometryCollectionArray		= 1 << 12,
		ActiveParticlesMapArray				= 1 << 13,
		TransientDirtyMapArray				= 1 << 14,
		MovingKinematicsMapArray			= 1 << 15,
		StaticClusteredMapArray				= 1 << 16,
		KinematicClusteredMapArray			= 1 << 18,
		DynamicClusteredMapArray			= 1 << 19,
		ResimActiveParticlesMapArray		= 1 << 20,
		ResimDynamicParticles				= 1 << 21,
		ResimDynamicKinematicParticles		= 1 << 22,
		ResimStaticParticles				= 1 << 23,
		ResimKinematicParticles				= 1 << 24,
	};
	ENUM_CLASS_FLAGS(EGeometryParticleListMask);
}