// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Box.h"
#include "Chaos/Capsule.h"
#include "Chaos/CastingUtilities.h"
#include "Chaos/Convex.h"
#include "Chaos/GJK.h"
#include "Chaos/HeightField.h"
#include "Chaos/ImplicitObject.h"
#include "Chaos/ImplicitObjectScaled.h"
#include "Chaos/ImplicitObjectUnion.h"
#include "Chaos/Levelset.h"
#include "Chaos/Plane.h"
#include "Chaos/Sphere.h"
#include "Chaos/Transform.h"
#include "Chaos/TriangleMeshImplicitObject.h"

#include "ChaosArchive.h"
#include <algorithm>
#include <utility>

namespace Chaos
{

	struct FMTDInfo
	{
		FVec3 Normal;
		FVec3 Position;
		FReal Penetration;
	};

	template <typename QueryGeometry>
	bool OverlapQuery(const FImplicitObject& A, const FRigidTransform3& ATM, const QueryGeometry& B, const FRigidTransform3& BTM, const FReal Thickness = 0, FMTDInfo* OutMTD=nullptr)
	{
		const EImplicitObjectType AType = A.GetType();
		constexpr EImplicitObjectType BType = QueryGeometry::StaticType();
		
		if (AType == ImplicitObjectType::Transformed)
		{
			const TImplicitObjectTransformed<FReal, 3>& TransformedA = static_cast<const TImplicitObjectTransformed<FReal, 3>&>(A);
			const FRigidTransform3 NewATM = TransformedA.GetTransform() * ATM;
			return OverlapQuery(*TransformedA.GetTransformedObject(), NewATM, B, BTM, Thickness, OutMTD);
		}

		const FRigidTransform3 BToATM = BTM.GetRelativeTransform(ATM);

		// This specialization for sphere is bugged since the sphere radius is not inverse scaled, 
		// nor can it be properly if testing against non-uniform scaled convexes
		//if(BType == ImplicitObjectType::Sphere)
		//{
		//	const FImplicitObject& BBase = static_cast<const FImplicitObject&>(B);
		//	const FSphere& BSphere = static_cast<const FSphere&>(BBase);
		//	const FVec3 PtInA = BToATM.TransformPositionNoScale(BSphere.GetCenter());
		//	return A.Overlap(PtInA, Thickness + BSphere.GetRadiusf());
		//}
		////todo: A is a sphere
		//else 
		if (A.IsConvex())
		{
			const FVec3 Offset = ATM.GetLocation() - BTM.GetLocation();
			if (OutMTD)
			{
				return Utilities::CastHelper(A, BToATM, [&](const auto& AConcrete, const auto& BToAFullTM)
				{
					FVec3 LocalA, LocalB, LocalNormal;
					int32 ClosestVertexIndexA, ClosestVertexIndexB;
					if(GJKPenetration<false, FReal>(AConcrete,B,BToAFullTM,OutMTD->Penetration, LocalA, LocalB, LocalNormal,ClosestVertexIndexA,ClosestVertexIndexB,Thickness,0.,Offset.SizeSquared() < 1e-4 ? FVec3(1,0,0) : Offset))
					{
						OutMTD->Normal = ATM.TransformVectorNoScale(LocalNormal);
						OutMTD->Position = ATM.TransformPosition(LocalA);
						return true;
					}

					return false;
				});
			}
			else
			{
				return Utilities::CastHelper(A, BToATM, [&](const auto& AConcrete, const auto& BToAFullTM) { return GJKIntersection<FReal>(AConcrete, B, BToAFullTM, Thickness, Offset.SizeSquared() < 1e-4 ? FVec3(1, 0, 0) : Offset); });
			}
		}
		else
		{
			bool bOverlap = false;
			switch (AType)
			{
				case ImplicitObjectType::HeightField:
				{
					const FHeightField& AHeightField = static_cast<const FHeightField&>(A);
					bOverlap = AHeightField.OverlapGeom(B, BToATM, Thickness, OutMTD);
					break;
				}
				case ImplicitObjectType::TriangleMesh:
				{
					const FTriangleMeshImplicitObject& ATriangleMesh = static_cast<const FTriangleMeshImplicitObject&>(A);
					bOverlap = ATriangleMesh.OverlapGeom(B, BToATM, Thickness, OutMTD);
					break;
				}
				case ImplicitObjectType::LevelSet:
				{
					const FLevelSet& ALevelSet = static_cast<const FLevelSet&>(A);
					bOverlap = ALevelSet.OverlapGeom(B, BToATM, Thickness, OutMTD);
					break;
				}
				case ImplicitObjectType::Union:
				case ImplicitObjectType::UnionClustered:
				{
					const FImplicitObjectUnion& AUnion = static_cast<const FImplicitObjectUnion&>(A);
					bool bHit = false;
					AUnion.ForEachObject(
						[&bHit, &ATM, &B, &BTM, Thickness, &OutMTD](const FImplicitObject& SubObject, const FRigidTransform3& SubTransform)
						{
							const FRigidTransform3 NewATM = SubTransform * ATM;
							if (OverlapQuery(SubObject, NewATM, B, BTM, Thickness, OutMTD))
							{
								bHit = true;
								return true;
							}

							return false;
						}
					);

					return bHit;
				}
				default:
				{
					if(IsScaled(AType))
					{
						const auto& AScaled = TImplicitObjectScaled<FTriangleMeshImplicitObject>::AsScaledChecked(A);
						bOverlap =  AScaled.LowLevelOverlapGeom(B, BToATM, Thickness, OutMTD);
					}
					else if(IsInstanced(AType))
					{
						const auto& AInstanced = TImplicitObjectInstanced<FTriangleMeshImplicitObject>::AsInstancedChecked(A);
						bOverlap = AInstanced.LowLevelOverlapGeom(B, BToATM, Thickness, OutMTD);
					}
					else
					{
						check(false);	//unsupported query type
					}
				}
			}

			if (OutMTD && bOverlap)
			{
				OutMTD->Normal = ATM.TransformVectorNoScale(OutMTD->Normal);
				OutMTD->Position = ATM.TransformPosition(OutMTD->Position);
			}
			return bOverlap;
		}
	}

	UE_INTERNAL CHAOS_API void TransformSweepResultsToWorld(const bool bResult, const FReal Time, const bool bComputeMTD, const FImplicitObject& TestGeom, const FRigidTransform3& TestGeomTM, const FVec3& LocalDir, const FVec3& LocalPosition, const FVec3& LocalNormal, const int32 FaceIndex, FVec3& OutWorldPosition, FVec3& OutWorldNormal, FVec3& OutWorldFaceNormal);
	UE_INTERNAL CHAOS_API bool SweepSphereVsSphere(const FSphere& SweptSphere, const FRigidTransform3& SweptSphereTM, const FSphere& TestSphere, const FRigidTransform3& TestSphereTM, const FVec3& SweepDir, const FReal Length, const FReal Thickness, const bool bComputeMTD, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal);
	UE_INTERNAL CHAOS_API bool SweepSphereVsBox(const FSphere& SweptSphere, const FRigidTransform3& SweptSphereTM, const TBox<FReal, 3>& TestBox, const FRigidTransform3& TestBoxTM, const FVec3& SweepDir, const FReal Length, const FReal Thickness, const bool bComputeMTD, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal);
	UE_INTERNAL CHAOS_API bool SweepSphereVsCapsule(const FSphere& SweptSphere, const FRigidTransform3& SweptSphereTM, const FCapsule& TestCapsule, const FRigidTransform3& TestCapsuleTM, const FVec3& SweepDir, const FReal Length, const FReal Thickness, const bool bComputeMTD, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal);
	UE_INTERNAL CHAOS_API bool SweepSphereVsConvex(const FSphere& SweptSphere, const FRigidTransform3& SweptSphereTM, const FImplicitObject& TestObject, const FRigidTransform3& TestObjectTM, const FVec3& SweepDir, const FReal Length, const FReal Thickness, const bool bComputeMTD, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal);
	UE_INTERNAL CHAOS_API bool SweepBoxVsSphere(const TBox<FReal, 3>& SweptBox, const FRigidTransform3& SweptBoxTM, const FSphere& TestSphere, const FRigidTransform3& TestSphereTM, const FVec3& SweepDir, const FReal Length, const FReal Thickness, const bool bComputeMTD, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal);
	UE_INTERNAL CHAOS_API bool SweepBoxVsCapsule(const TBox<FReal, 3>& SweptBox, const FRigidTransform3& SweptBoxTM, const FCapsule& TestCapsule, const FRigidTransform3& TestCapsuleTM, const FVec3& SweepDir, const FReal Length, const FReal Thickness, const bool bComputeMTD, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal);
	UE_INTERNAL CHAOS_API bool SweepCapsuleVsSphere(const FCapsule& SweptCapsule, const FRigidTransform3& SweptCapsuleTM, const FSphere& TestSphere, const FRigidTransform3& TestSphereTM, const FVec3& SweepDir, const FReal Length, const FReal Thickness, const bool bComputeMTD, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal);
	UE_INTERNAL CHAOS_API bool SweepCapsuleVsBox(const FCapsule& SweptCapsule, const FRigidTransform3& SweptCapsuleTM, const TBox<FReal, 3>& TestBox, const FRigidTransform3& TestBoxTM, const FVec3& SweepDir, const FReal Length, const FReal Thickness, const bool bComputeMTD, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal);
	UE_INTERNAL CHAOS_API bool SweepCapsuleVsCapsule(const FCapsule& SweptCapsule, const FRigidTransform3& SweptCapsuleTM, const FCapsule& TestCapsule, const FRigidTransform3& TestCapsuleTM, const FVec3& SweepDir, const FReal Length, const FReal Thickness, const bool bComputeMTD, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal);
	UE_INTERNAL CHAOS_API bool SweepCapsuleVsConvex(const FCapsule& SweptCapsule, const FRigidTransform3& SweptCapsuleTM, const FImplicitObject& TestObject, const FRigidTransform3& TestObjectTM, const FVec3& SweepDir, const FReal Length, const FReal Thickness, const bool bComputeMTD, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal);

	template <EImplicitObjectType SweptType>
	UE_INTERNAL bool TryRunSpecializedSweep(const FImplicitObject& SweptShape, const FRigidTransform3& SweptShapeTM, const FImplicitObject& TestGeom, const FRigidTransform3& TestGeomTM, const FVec3& SweepDir, const FReal Length, const FReal Thickness, const bool bComputeMTD, bool& bOutResult, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal)
	{
		const EImplicitObjectType TestType = TestGeom.GetType();
		if (SweptType == ImplicitObjectType::Sphere)
		{
			if (TestType == ImplicitObjectType::Sphere)
			{
				const FSphere& SweptSphere = SweptShape.GetObjectChecked<FSphere>();
				const FSphere& TestSphere = TestGeom.GetObjectChecked<FSphere>();
				bOutResult = SweepSphereVsSphere(SweptSphere, SweptShapeTM, TestSphere, TestGeomTM, SweepDir, Length, Thickness, bComputeMTD, OutTime, OutPosition, OutNormal, OutFaceIndex, OutFaceNormal);
				return true;
			}
			else if (TestType == ImplicitObjectType::Box)
			{
				const FSphere& SweptSphere = SweptShape.GetObjectChecked<FSphere>();
				const TBox<FReal, 3>& TestBox = TestGeom.GetObjectChecked<TBox<FReal, 3>>();
				bOutResult = SweepSphereVsBox(SweptSphere, SweptShapeTM, TestBox, TestGeomTM, SweepDir, Length, Thickness, bComputeMTD, OutTime, OutPosition, OutNormal, OutFaceIndex, OutFaceNormal);
				return true;
			}
			else if (TestType == ImplicitObjectType::Capsule)
			{
				const FSphere& SweptSphere = SweptShape.GetObjectChecked<FSphere>();
				const FCapsule& TestCapsule = TestGeom.GetObjectChecked<FCapsule>();
				bOutResult = SweepSphereVsCapsule(SweptSphere, SweptShapeTM, TestCapsule, TestGeomTM, SweepDir, Length, Thickness, bComputeMTD, OutTime, OutPosition, OutNormal, OutFaceIndex, OutFaceNormal);
				return true;
			}
			else if (TestGeom.IsConvex())
			{
				const FSphere& SweptSphere = SweptShape.GetObjectChecked<FSphere>();
				bOutResult = SweepSphereVsConvex(SweptSphere, SweptShapeTM, TestGeom, TestGeomTM, SweepDir, Length, Thickness, bComputeMTD, OutTime, OutPosition, OutNormal, OutFaceIndex, OutFaceNormal);
				return true;
			}
		}
		else if (SweptType == ImplicitObjectType::Box)
		{
			if (TestType == ImplicitObjectType::Sphere)
			{
				const TBox<FReal, 3>& SweptBox = SweptShape.GetObjectChecked<TBox<FReal, 3>>();
				const FSphere& TestSphere = TestGeom.GetObjectChecked<FSphere>();
				bOutResult = SweepBoxVsSphere(SweptBox, SweptShapeTM, TestSphere, TestGeomTM, SweepDir, Length, Thickness, bComputeMTD, OutTime, OutPosition, OutNormal, OutFaceIndex, OutFaceNormal);
				return true;
			}
			else if (TestType == ImplicitObjectType::Capsule)
			{
				const TBox<FReal, 3>& SweptBox = SweptShape.GetObjectChecked<TBox<FReal, 3>>();
				const FCapsule& TestCapsule = TestGeom.GetObjectChecked<FCapsule>();
				bOutResult = SweepBoxVsCapsule(SweptBox, SweptShapeTM, TestCapsule, TestGeomTM, SweepDir, Length, Thickness, bComputeMTD, OutTime, OutPosition, OutNormal, OutFaceIndex, OutFaceNormal);
				return true;
			}
		}
		else if (SweptType == ImplicitObjectType::Capsule)
		{
			if (TestType == ImplicitObjectType::Sphere)
			{
				const FCapsule& SweptCapsule = SweptShape.GetObjectChecked<FCapsule>();
				const FSphere& TestSphere = TestGeom.GetObjectChecked<FSphere>();
				bOutResult = SweepCapsuleVsSphere(SweptCapsule, SweptShapeTM, TestSphere, TestGeomTM, SweepDir, Length, Thickness, bComputeMTD, OutTime, OutPosition, OutNormal, OutFaceIndex, OutFaceNormal);
				return true;
			}
			else if (TestType == ImplicitObjectType::Box)
			{
				const FCapsule& SweptCapsule = SweptShape.GetObjectChecked<FCapsule>();
				const TBox<FReal, 3>& TestBox = TestGeom.GetObjectChecked<TBox<FReal, 3>>();
				bOutResult = SweepCapsuleVsBox(SweptCapsule, SweptShapeTM, TestBox, TestGeomTM, SweepDir, Length, Thickness, bComputeMTD, OutTime, OutPosition, OutNormal, OutFaceIndex, OutFaceNormal);
				return true;
			}
			else if (TestType == ImplicitObjectType::Capsule)
			{
				const FCapsule& SweptCapsule = SweptShape.GetObjectChecked<FCapsule>();
				const FCapsule& TestCapsule = TestGeom.GetObjectChecked<FCapsule>();
				bOutResult = SweepCapsuleVsCapsule(SweptCapsule, SweptShapeTM, TestCapsule, TestGeomTM, SweepDir, Length, Thickness, bComputeMTD, OutTime, OutPosition, OutNormal, OutFaceIndex, OutFaceNormal);
				return true;
			}
			else if (TestGeom.IsConvex())
			{
				const FCapsule& SweptCapsule = SweptShape.GetObjectChecked<FCapsule>();
				bOutResult = SweepCapsuleVsConvex(SweptCapsule, SweptShapeTM, TestGeom, TestGeomTM, SweepDir, Length, Thickness, bComputeMTD, OutTime, OutPosition, OutNormal, OutFaceIndex, OutFaceNormal);
				return true;
			}
		}
		return false;
	}

	template <typename SweptGeometry>
	bool SweepQuery(const FImplicitObject& A, const FRigidTransform3& ATM, const SweptGeometry& B, const FRigidTransform3& BTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FReal Thickness, const bool bComputeMTD)
	{
		const EImplicitObjectType AType = A.GetType();
		constexpr EImplicitObjectType BType = SweptGeometry::StaticType();

		bool bResult = false;
		
		if (AType == ImplicitObjectType::Transformed)
		{
			const TImplicitObjectTransformed<FReal, 3>& TransformedA = static_cast<const TImplicitObjectTransformed<FReal, 3>&>(A);
			const FRigidTransform3 NewATM = TransformedA.GetTransform() * ATM;
			return SweepQuery(*TransformedA.GetTransformedObject(), NewATM, B, BTM, Dir, Length, OutTime, OutPosition, OutNormal, OutFaceIndex, OutFaceNormal, Thickness, bComputeMTD);
		}

		OutFaceIndex = INDEX_NONE;
		PRAGMA_DISABLE_INTERNAL_WARNINGS
		if (TryRunSpecializedSweep<BType>(B, BTM, A, ATM, Dir, Length, Thickness, bComputeMTD, bResult, OutTime, OutPosition, OutNormal, OutFaceIndex, OutFaceNormal))
		{
			return bResult;
		}
		PRAGMA_ENABLE_INTERNAL_WARNINGS

		FVec3 LocalPosition(-TNumericLimits<FReal>::Max()); // Make it obvious when things go wrong
		FVec3 LocalNormal(0);

		const FRigidTransform3 BToATM = BTM.GetRelativeTransformNoScale(ATM);
		ensure(FMath::IsNearlyEqual(Dir.SizeSquared(), (FReal)1, (FReal)UE_KINDA_SMALL_NUMBER)); // Added to help determine cause of this ensure firing in GJKRaycast2.
		const FVec3 LocalDir = ATM.InverseTransformVectorNoScale(Dir);

		// We can optimize some sweeps as a raycast. Sweeps can have zero length and most raycast functions don't handle this.
		// Do not allow unions to do this optimization. A union could contain a non-uniform scaled object which would break the optimization.
		// Each internal object in the union will do the optimization if possible.
		bool bSweepAsRaycast = BType == ImplicitObjectType::Sphere && !bComputeMTD && Length != 0 && (AType != ImplicitObjectType::Union && AType != ImplicitObjectType::UnionClustered);
		if (bSweepAsRaycast && IsScaled(AType))
		{
			const auto& Scaled = TImplicitObjectScaledGeneric<FReal, 3>::AsScaledChecked(A);
			const FVec3& Scale = Scaled.GetScale();
			bSweepAsRaycast = FMath::IsNearlyEqual(Scale[0], Scale[1]) && FMath::IsNearlyEqual(Scale[0], Scale[2]);
		}

		if (bSweepAsRaycast)
		{
			const FImplicitObject& BBase = B;
			const FSphere& BSphere = BBase.GetObjectChecked<FSphere>();
			const FVec3 Start = BToATM.TransformPositionNoScale(FVec3(BSphere.GetCenterf()));
			bResult = A.Raycast(Start, LocalDir, Length, Thickness + BSphere.GetRadiusf(), OutTime, LocalPosition, LocalNormal, OutFaceIndex);
		}
		//todo: handle case where A is a sphere
		else if (A.IsConvex())
		{
			auto IsValidConvex = [](const FImplicitObject& InObject) -> bool
			{
				//todo: move this out of here
				if (const auto Convex = TImplicitObjectScaled<FConvex>::AsScaled(InObject))
				{
					return Convex->GetUnscaledObject()->NumVertices() > 0;
				}				

				return true;
			};

			// Validate that the convexes we are about to test are actually valid geometries
			if(!ensureMsgf(IsValidConvex(A), TEXT("GJKRaycast - Convex A has no particles")) ||
				!ensureMsgf(IsValidConvex(B), TEXT("GJKRaycast - Convex B has no particles")))
			{
				return false;
			}

			const FVec3 Offset = ATM.GetLocation() - BTM.GetLocation();
			bResult = Utilities::CastHelperNoUnwrap(A, BToATM, 
				[&](const auto& ADowncast, const auto& BToAFullTM)
				{
					return GJKRaycast2(ADowncast, B, BToAFullTM, LocalDir, Length, OutTime, LocalPosition, LocalNormal, Thickness, bComputeMTD, Offset, Thickness);
				});

			if (AType == ImplicitObjectType::Convex)
			{
				//todo: find face index
			}
			else if (AType == ImplicitObjectType::DEPRECATED_Scaled)
			{
				ensure(false);
				//todo: find face index if convex hull
			}
		}
		else
		{
			switch (AType)
			{
				case ImplicitObjectType::HeightField:
				{
					const FHeightField& AHeightField = static_cast<const FHeightField&>(A);
					bResult = AHeightField.SweepGeom(B, BToATM, LocalDir, Length, OutTime, LocalPosition, LocalNormal, OutFaceIndex, OutFaceNormal, Thickness, bComputeMTD);
					break;
				}
				case ImplicitObjectType::TriangleMesh:
				{
					const FTriangleMeshImplicitObject& ATriangleMesh = static_cast<const FTriangleMeshImplicitObject&>(A);
					bResult = ATriangleMesh.SweepGeom(B, BToATM, LocalDir, Length, OutTime, LocalPosition, LocalNormal, OutFaceIndex, OutFaceNormal, Thickness, bComputeMTD);
					break;
				}
				case ImplicitObjectType::LevelSet:
				{
					const FLevelSet& ALevelSet = static_cast<const FLevelSet&>(A);
					bResult = ALevelSet.SweepGeom(B, BToATM, LocalDir, Length, OutTime, LocalPosition, LocalNormal, OutFaceIndex, Thickness, bComputeMTD);
					break;
				}
				case ImplicitObjectType::Union:
				case ImplicitObjectType::UnionClustered:
				{
					const FImplicitObjectUnion& AUnion = static_cast<const FImplicitObjectUnion&>(A);

					bool bHit = false;
					OutTime = TNumericLimits<FReal>::Max();
					AUnion.ForEachObject(
						[&bHit, &ATM, &B, &BTM, &Dir, Length, &OutTime, &OutPosition, &OutNormal, &OutFaceIndex, &OutFaceNormal, Thickness, bComputeMTD](const FImplicitObject& SubObject, const FRigidTransform3& SubTransform)
						{
							const FRigidTransform3 NewATM = SubTransform * ATM;

							FReal ObjectTime = 0.0;
							FVec3 ObjectPosition;
							FVec3 ObjectNormal;
							int32 ObjectFaceIndex;
							FVec3 ObjectFaceNormal;

							if (SweepQuery(SubObject, NewATM, B, BTM, Dir, Length, ObjectTime, ObjectPosition, ObjectNormal, ObjectFaceIndex, ObjectFaceNormal, Thickness, bComputeMTD))
							{
								bHit = true;
								if (ObjectTime < OutTime)
								{
									OutTime = ObjectTime;
									OutPosition = ObjectPosition;
									OutNormal = ObjectNormal;
									OutFaceIndex = ObjectFaceIndex;
									OutFaceNormal = ObjectFaceNormal;
								}
							}

							return false;
						}
					);

					return bHit;
				}
				default:
				if (IsScaled(AType))
				{
					const auto& AScaled = TImplicitObjectScaled<FTriangleMeshImplicitObject>::AsScaledChecked(A);
					bResult = AScaled.LowLevelSweepGeom(B, BToATM, LocalDir, Length, OutTime, LocalPosition, LocalNormal, OutFaceIndex, OutFaceNormal, Thickness, bComputeMTD);
					break;
				}
				else if(IsInstanced(AType))
				{
					const auto& Instanced = TImplicitObjectInstanced<FTriangleMeshImplicitObject>::AsInstancedChecked(A);
					bResult = Instanced.LowLevelSweepGeom(B, BToATM, LocalDir, Length, OutTime, LocalPosition, LocalNormal, OutFaceIndex, OutFaceNormal, Thickness, bComputeMTD);
					break;
				}
				else
				{
					ensureMsgf(false, TEXT("Unsupported query type: %u"), (uint8)AType);
				}
			}
		}

		//put back into world space
		PRAGMA_DISABLE_INTERNAL_WARNINGS
		TransformSweepResultsToWorld(bResult, OutTime, bComputeMTD, A, ATM, LocalDir, LocalPosition, LocalNormal, OutFaceIndex, OutPosition, OutNormal, OutFaceNormal);
		PRAGMA_ENABLE_INTERNAL_WARNINGS

		return bResult;
	}


	// @todo(chaos): This does not handle Unions
	inline bool SweepQuery(const FImplicitObject& A, const FRigidTransform3& ATM, const FImplicitObject& B, const FRigidTransform3& BTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FReal Thickness, const bool bComputeMTD)
	{
		return Chaos::Utilities::CastHelper(B, BTM,
			[&A, &ATM, &Dir, &Length, &OutTime, &OutPosition, &OutNormal, &OutFaceIndex, &OutFaceNormal, &Thickness, &bComputeMTD]
			(const auto& BInner, const FTransform& BInnerTM) -> bool
			{
				return SweepQuery(A, ATM, BInner, BInnerTM, Dir, Length, OutTime, OutPosition, OutNormal, OutFaceIndex, OutFaceNormal, Thickness, bComputeMTD);
			});
	}

}
