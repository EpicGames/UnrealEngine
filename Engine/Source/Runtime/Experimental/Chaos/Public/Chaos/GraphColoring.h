// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Array.h"
#include "Chaos/DynamicParticles.h"
#include "Chaos/Core.h"
#include "Chaos/Vector.h"
#include "Chaos/UniformGrid.h"
#include "Chaos/PBDActiveView.h"
#include "Chaos/SoftsSolverParticlesRange.h"

namespace Chaos
{

class FGraphColoring
{
	typedef TArray<int32, TInlineAllocator<8>> FColorSet;

  public:

	template<typename DynamicParticlesType, int32 N, bool bAllDynamic = false>
	static CHAOS_API TArray<TArray<int32>> ComputeGraphColoringParticlesOrRange(const TArray<TVector<int32,N>>& Graph, const DynamicParticlesType& InParticles, const int32 GraphParticlesStart, const int32 GraphParticlesEnd);

	template<typename T, int32 N>
	inline static TArray<TArray<int32>> ComputeGraphColoring(const TArray<TVector<int32, N>>& Graph, const TDynamicParticles<T, 3>& InParticles, const int32 GraphParticlesStart, const int32 GraphParticlesEnd)
	{
		return ComputeGraphColoringParticlesOrRange(Graph, InParticles, GraphParticlesStart, GraphParticlesEnd);
	}
	template<typename T, int32 N>
	inline static TArray<TArray<int32>> ComputeGraphColoring(const TArray<TVector<int32, N>>& Graph, const TDynamicParticles<T, 3>& InParticles)
	{
		return ComputeGraphColoringParticlesOrRange(Graph, InParticles, 0, InParticles.Size());
	}

	template<typename DynamicParticlesType>
	inline static TArray<TArray<int32>> ComputeGraphColoringAllDynamicParticlesOrRange(const TArray<TVec4<int32>>& Graph, const DynamicParticlesType& InParticles, const int32 GraphParticlesStart, const int32 GraphParticlesEnd)
	{
		return ComputeGraphColoringParticlesOrRange<DynamicParticlesType, 4, true>(Graph, InParticles, GraphParticlesStart, GraphParticlesEnd);
	}
	template<typename T>
	inline static TArray<TArray<int32>> ComputeGraphColoringAllDynamic(const TArray<TVec4<int32>>& Graph, const Chaos::TDynamicParticles<T, 3>& InParticles, const int32 GraphParticlesStart, const int32 GraphParticlesEnd)
	{
		return ComputeGraphColoringAllDynamicParticlesOrRange(Graph, InParticles, GraphParticlesStart, GraphParticlesEnd);
	}
	template<typename T>
	inline static TArray<TArray<int32>> ComputeGraphColoringAllDynamic(const TArray<TVec4<int32>>& Graph, const Chaos::TDynamicParticles<T, 3>& InParticles)
	{
		return ComputeGraphColoringAllDynamicParticlesOrRange(Graph, InParticles, 0, InParticles.Size());
	}
};

template<typename T> 
UE_DEPRECATED(5.6, "Use UniquePtr version instead.")
void ComputeGridBasedGraphSubColoringPointer(const TArray<TArray<int32>>& ElementsPerColor, const TMPMGrid<T>& Grid, const int32 GridSize, TArray<TArray<int32>>*& PreviousColoring, const TArray<TArray<int32>>& ConstraintsNodesSet, TArray<TArray<TArray<int32>>>& ElementsPerSubColors)
{
	TUniquePtr<TArray<TArray<int32>>> PreviousColoringUniquePtr(PreviousColoring);
	ComputeGridBasedGraphSubColoringPointer(ElementsPerColor, Grid, GridSize, PreviousColoringUniquePtr, ConstraintsNodesSet, ElementsPerSubColors);
	PreviousColoring = PreviousColoringUniquePtr.Release();
}

template<typename T>
CHAOS_API void ComputeGridBasedGraphSubColoringPointer(const TArray<TArray<int32>>& ElementsPerColor, const TMPMGrid<T>& Grid, const int32 GridSize, TUniquePtr<TArray<TArray<int32>>>& PreviousColoring, const TArray<TArray<int32>>& ConstraintsNodesSet, TArray<TArray<TArray<int32>>>& ElementsPerSubColors);

template<typename T>
UE_DEPRECATED(5.8, "Use the NumParticles overload instead of passing TDynamicParticles.")
CHAOS_API void ComputeWeakConstraintsColoring(const TArray<TArray<int32>>& Indices, const TArray<TArray<int32>>& SecondIndices, const Chaos::TDynamicParticles<T, 3>& InParticles, TArray<TArray<int32>>& ConstraintsPerColor);

// Non-template overloads taking particle count directly (for use with FSolverParticlesRange)
// Use this version if Indices and SecondIndices point to the same particles
CHAOS_API void ComputeWeakConstraintsColoring(const TArray<TArray<int32>>& Indices, const TArray<TArray<int32>>& SecondIndices, int32 NumParticles, TArray<TArray<int32>>& ConstraintsPerColor);
// Use this version if Indices and SecondIndices point to different particles, correspondingly
CHAOS_API void ComputeWeakConstraintsColoring(const TArray<TArray<int32>>& Indices, const TArray<TArray<int32>>& SecondIndices, int32 NumParticles, int32 NumSecondParticles, TArray<TArray<int32>>& ConstraintsPerColor);

template<typename T>
CHAOS_API TArray<TArray<int32>> ComputeNodalColoring(const TArray<TVec4<int32>>& Graph, const Chaos::TDynamicParticles<T, 3>& InParticles, const int32 GraphParticlesStart, const int32 GraphParticlesEnd, const TArray<TArray<int32>>& IncidentElements, const TArray<TArray<int32>>& IncidentElementsLocalIndex);

template<typename T, typename ParticleType>
CHAOS_API TArray<TArray<int32>> ComputeNodalColoring(const TArray<TArray<int32>>& Graph, const Chaos::TDynamicParticles<T, 3>& InParticles, const int32 GraphParticlesStart, const int32 GraphParticlesEnd, const TArray<TArray<int32>>& IncidentElements, const TArray<TArray<int32>>& IncidentElementsLocalIndex, const Chaos::TPBDActiveView<ParticleType>* InParticleActiveView = nullptr, TArray<int32>* ParticleColorsOut = nullptr);

template<typename T>
CHAOS_API void ComputeExtraNodalColoring(const TArray<TArray<int32>>& Graph, const TArray<TArray<int32>>& ExtraGraph, const Chaos::TDynamicParticles<T, 3>& InParticles, const TArray<TArray<int32>>& IncidentElements, const TArray<TArray<int32>>& ExtraIncidentElements, TArray<int32>& ParticleColors, TArray<TArray<int32>>& ParticlesPerColor);

template<typename T>
CHAOS_API void ComputeExtraNodalColoring(const TArray<TArray<int32>>& StaticGraph, const TArray<TArray<int32>>& DynamicGraph, const TArray<TArray<int32>>& ExtraGraph, const Chaos::TDynamicParticles<T, 3>& InParticles, const TArray<TArray<int32>>& StaticIncidentElements, const TArray<TArray<int32>>& DynamicIncidentElements, const TArray<TArray<int32>>& ExtraIncidentElements, TArray<int32>& ParticleColors, TArray<TArray<int32>>& ParticlesPerColor);

// FSolverParticlesRange versions
CHAOS_API TArray<TArray<int32>> ComputeNodalColoring(const TArray<TVec4<int32>>& Graph, const Softs::FSolverParticlesRange& InParticles, const int32 GraphParticlesStart, const int32 GraphParticlesEnd, const TArray<TArray<int32>>& IncidentElements, const TArray<TArray<int32>>& IncidentElementsLocalIndex);

template<typename ParticleType>
CHAOS_API TArray<TArray<int32>> ComputeNodalColoring(const TArray<TArray<int32>>& Graph, const Softs::FSolverParticlesRange& InParticles, const int32 GraphParticlesStart, const int32 GraphParticlesEnd, const TArray<TArray<int32>>& IncidentElements, const TArray<TArray<int32>>& IncidentElementsLocalIndex, const Chaos::TPBDActiveView<ParticleType>* InParticleActiveView = nullptr, TArray<int32>* ParticleColorsOut = nullptr);

CHAOS_API void ComputeExtraNodalColoring(const TArray<TArray<int32>>& Graph, const TArray<TArray<int32>>& ExtraGraph, const Softs::FSolverParticlesRange& InParticles, const TArray<TArray<int32>>& IncidentElements, const TArray<TArray<int32>>& ExtraIncidentElements, TArray<int32>& ParticleColors, TArray<TArray<int32>>& ParticlesPerColor);

CHAOS_API void ComputeExtraNodalColoring(const TArray<TArray<int32>>& StaticGraph, const TArray<TArray<int32>>& DynamicGraph, const TArray<TArray<int32>>& ExtraGraph, const Softs::FSolverParticlesRange& InParticles, const TArray<TArray<int32>>& StaticIncidentElements, const TArray<TArray<int32>>& DynamicIncidentElements, const TArray<TArray<int32>>& ExtraIncidentElements, TArray<int32>& ParticleColors, TArray<TArray<int32>>& ParticlesPerColor);


}
