// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Array.h"
#include "Chaos/CollisionResolutionTypes.h"
#include "ImplicitObject.h"
#include "Box.h"
#include "TriangleMeshImplicitObject.h"
#include "ChaosArchive.h"
#include "Serialization/MemoryWriter.h"
#include "Math/NumericLimits.h"
#include "Templates/UniqueObj.h"
#include "Templates/UnrealTypeTraits.h"
#include "UniformGrid.h"
#include "Utilities.h"
#include "UObject/ExternalPhysicsCustomObjectVersion.h"

namespace Chaos
{
	class FHeightfieldRaycastVisitor;
	struct FMTDInfo;
}

namespace Chaos::Private
{
	class FMeshContactGenerator;
}

namespace UE::Physics
{
	class FHeightFieldGeometry;
}

namespace Chaos
{
	class FHeightField final : public FImplicitObject
	{
	public:
		using FImplicitObject::GetTypeName;

		CHAOS_API FHeightField(TArray<FReal>&& Height, TArray<uint8>&& InMaterialIndices, int32 InNumRows, int32 InNumCols, const FVec3& InScale);
		CHAOS_API FHeightField(TArrayView<const uint16> InHeights, TArrayView<const uint8> InMaterialIndices, int32 InNumRows, int32 InNumCols, const FVec3& InScale);
		FHeightField(const FHeightField& Other) = delete;
		
		// Not required as long as FImplicitObject also has deleted move constructor (adding this causes an error on Linux build)
		//FHeightField(FHeightField&& Other) = default;

		virtual ~FHeightField() = default;

		/** Support for editing a subsection of the heightfield */
		CHAOS_API void EditHeights(TArrayView<FReal> InHeights, int32 InBeginRow, int32 InBeginCol, int32 InNumRows, int32 InNumCols);
		CHAOS_API void EditHeights(TArrayView<const uint16> InHeights, int32 InBeginRow, int32 InBeginCol, int32 InNumRows, int32 InNumCols);
		CHAOS_API FReal GetHeight(int32 InIndex) const;
		CHAOS_API FVec3 GetPointScaled(int32 InIndex) const;
		CHAOS_API FReal GetHeight(int32 InX, int32 InY) const;
		CHAOS_API uint8 GetMaterialIndex(int32 InIndex) const;
		CHAOS_API uint8 GetMaterialIndex(int32 InX, int32 InY) const;
		CHAOS_API bool IsHole(int32 InIndex) const;
		CHAOS_API bool IsHole(int32 InCellX, int32 InCellY) const;
		CHAOS_API FVec3 GetNormalAt(const FVec2& InGridLocationLocal) const;
		CHAOS_API FReal GetHeightAt(const FVec2& InGridLocationLocal) const;
		CHAOS_API uint8 GetMaterialIndexAt(const FVec2& InGridLocationLocal) const;

		int32 GetNumRows() const { return GeomData.NumRows; }
		int32 GetNumCols() const { return GeomData.NumCols; }

		CHAOS_API virtual FReal PhiWithNormal(const FVec3& x, FVec3& Normal) const;

		CHAOS_API virtual bool Raycast(const FVec3& StartPoint, const FVec3& Dir, const FReal Length, const FReal Thickness, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex) const override;
		CHAOS_API virtual bool Overlap(const FVec3& Point, const FReal Thickness) const override;
		
		CHAOS_API bool OverlapGeom(const FSphere& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD = nullptr) const;
		CHAOS_API bool OverlapGeom(const TBox<FReal, 3>& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD = nullptr) const;
		CHAOS_API bool OverlapGeom(const FCapsule& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD = nullptr) const;
		CHAOS_API bool OverlapGeom(const FConvex& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD = nullptr) const;
		CHAOS_API bool OverlapGeom(const TImplicitObjectScaled<FSphere>& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD = nullptr) const;
		CHAOS_API bool OverlapGeom(const TImplicitObjectScaled<TBox<FReal, 3>>& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD = nullptr) const;
		CHAOS_API bool OverlapGeom(const TImplicitObjectScaled<FCapsule>& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD = nullptr) const;
		CHAOS_API bool OverlapGeom(const TImplicitObjectScaled<FConvex>& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD = nullptr) const;

		CHAOS_API bool SweepGeom(const FSphere& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FReal Thickness = 0, bool bComputeMTD = false) const;
		CHAOS_API bool SweepGeom(const TBox<FReal, 3>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FReal Thickness = 0, bool bComputeMTD = false) const;
		CHAOS_API bool SweepGeom(const FCapsule& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FReal Thickness = 0, bool bComputeMTD = false) const;
		CHAOS_API bool SweepGeom(const FConvex& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FReal Thickness = 0, bool bComputeMTD = false) const;
		CHAOS_API bool SweepGeom(const TImplicitObjectScaled<FSphere>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FReal Thickness = 0, bool bComputeMTD = false) const;
		CHAOS_API bool SweepGeom(const TImplicitObjectScaled<TBox<FReal, 3>>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FReal Thickness = 0, bool bComputeMTD = false) const;
		CHAOS_API bool SweepGeom(const TImplicitObjectScaled<FCapsule>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FReal Thickness = 0, bool bComputeMTD = false) const;
		CHAOS_API bool SweepGeom(const TImplicitObjectScaled<FConvex>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FReal Thickness = 0, bool bComputeMTD = false) const;

		// Sweep used for CCD. Ignores triangles we penetrate by less than IgnorePenetration, and calculate the TOI for a depth of TargetPenetration. If both are zero, this is equivalent to SweepGeom
		CHAOS_API bool SweepGeomCCD(const FSphere& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& OutTOI, FReal& OutPhi, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal) const;
		CHAOS_API bool SweepGeomCCD(const TBox<FReal, 3>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& OutTOI, FReal& OutPhi, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal) const;
		CHAOS_API bool SweepGeomCCD(const FCapsule& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& OutTOI, FReal& OutPhi, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal) const;
		CHAOS_API bool SweepGeomCCD(const FConvex& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& OutTOI, FReal& OutPhi, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal) const;
		CHAOS_API bool SweepGeomCCD(const TImplicitObjectScaled<FSphere>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& OutTOI, FReal& OutPhi, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal) const;
		CHAOS_API bool SweepGeomCCD(const TImplicitObjectScaled<TBox<FReal, 3>>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& OutTOI, FReal& OutPhi, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal) const;
		CHAOS_API bool SweepGeomCCD(const TImplicitObjectScaled<FCapsule>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& OutTOI, FReal& OutPhi, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal) const;
		CHAOS_API bool SweepGeomCCD(const TImplicitObjectScaled<FConvex>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& OutTOI, FReal& OutPhi, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal) const;

		CHAOS_API bool GJKContactPoint(const TBox<FReal, 3>& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FVec3& ContactLocation, FVec3& ContactNormal, FReal& ContactPhi, int32& ContactFaceIndex) const;
		CHAOS_API bool GJKContactPoint(const FSphere& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FVec3& ContactLocation, FVec3& ContactNormal, FReal& ContactPhi, int32& ContactFaceIndex) const;
		CHAOS_API bool GJKContactPoint(const FCapsule& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FVec3& ContactLocation, FVec3& ContactNormal, FReal& ContactPhi, int32& ContactFaceIndex) const;
		CHAOS_API bool GJKContactPoint(const FConvex& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FVec3& ContactLocation, FVec3& ContactNormal, FReal& ContactPhi, int32& ContactFaceIndex) const;
		CHAOS_API bool GJKContactPoint(const TImplicitObjectScaled<TBox<FReal, 3>>& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FVec3& ContactLocation, FVec3& ContactNormal, FReal& ContactPhi, int32& ContactFaceIndex) const;
		CHAOS_API bool GJKContactPoint(const TImplicitObjectScaled<FSphere>& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FVec3& ContactLocation, FVec3& ContactNormal, FReal& ContactPhi, int32& ContactFaceIndex) const;
		CHAOS_API bool GJKContactPoint(const TImplicitObjectScaled<FCapsule>& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FVec3& ContactLocation, FVec3& ContactNormal, FReal& ContactPhi, int32& ContactFaceIndex) const;
		CHAOS_API bool GJKContactPoint(const TImplicitObjectScaled<FConvex>& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FVec3& ContactLocation, FVec3& ContactNormal, FReal& ContactPhi, int32& ContactFaceIndex) const;


		CHAOS_API virtual int32 FindMostOpposingFace(const FVec3& Position, const FVec3& UnitDir, int32 HintFaceIndex, FReal SearchDist) const override;
		CHAOS_API virtual FVec3 FindGeometryOpposingNormal(const FVec3& DenormDir, int32 FaceIndex, const FVec3& OriginalNormal) const override;

		/**
		 * @brief Generate the triangle at the specified index with the specified transform (including scale)
		 * @note does not correct winding for negative scales
		*/
		void GetTransformedTriangle(const int32 TriangleIndex, const FRigidTransform3& Transform, FTriangle& OutTriangle, int32& OutVertexIndex0, int32& OutVertexIndex1, int32& OutVertexIndex2) const
		{
			// Convert the triangle index into a cell index and extract the cell row/column
			// Reverse of FaceIndex0 = CellIndex * 2 + 0|1;
			const int32 CellIndex = TriangleIndex / 2;
			const int32 CellRowIndex = CellIndex / (GeomData.NumCols - 1);
			const int32 CellColIndex = CellIndex % (GeomData.NumCols - 1);

			// Calculate the vertex indices for the 4 cell corners
			const int32 CellVertexIndex = CellRowIndex * GeomData.NumCols + CellColIndex;
			const int32 VertexIndex0 = CellVertexIndex;
			const int32 VertexIndex1 = CellVertexIndex + 1;
			const int32 VertexIndex2 = CellVertexIndex + GeomData.NumCols;
			const int32 VertexIndex3 = CellVertexIndex + GeomData.NumCols + 1;

			// Get the vertices
			const FVec3 Vertex0 = Transform.TransformPosition(GeomData.GetPointScaled(VertexIndex0));
			const FVec3 Vertex1 = Transform.TransformPosition(GeomData.GetPointScaled(VertexIndex1));
			const FVec3 Vertex2 = Transform.TransformPosition(GeomData.GetPointScaled(VertexIndex2));
			const FVec3 Vertex3 = Transform.TransformPosition(GeomData.GetPointScaled(VertexIndex3));

			// Set the output triangle which depends on winding (begative scales) and which of the two triangles in the cell we want
			const bool bStandardWinding = ((GeomData.Scale.X * GeomData.Scale.Y * GeomData.Scale.Z) >= FReal(0));
			const bool bIsFirstTriangle = ((TriangleIndex & 1) == 0);
			if (bIsFirstTriangle)
			{
				if (bStandardWinding)
				{
					OutTriangle = FTriangle(Vertex0, Vertex1, Vertex3);
					OutVertexIndex0 = VertexIndex0;
					OutVertexIndex1 = VertexIndex1;
					OutVertexIndex2 = VertexIndex3;
				}
				else
				{
					OutTriangle = FTriangle(Vertex0, Vertex3, Vertex1);
					OutVertexIndex0 = VertexIndex0;
					OutVertexIndex1 = VertexIndex3;
					OutVertexIndex2 = VertexIndex1;
				}
			}
			else
			{
				if (bStandardWinding)
				{
					OutTriangle = FTriangle(Vertex0, Vertex3, Vertex2);
					OutVertexIndex0 = VertexIndex0;
					OutVertexIndex1 = VertexIndex3;
					OutVertexIndex2 = VertexIndex2;
				}
				else
				{
					OutTriangle = FTriangle(Vertex0, Vertex2, Vertex3);
					OutVertexIndex0 = VertexIndex0;
					OutVertexIndex1 = VertexIndex2;
					OutVertexIndex2 = VertexIndex3;
				}
			}
		}

		/**
		 * Visit all triangles transformed into the specified space. Triangles will have standard winding, regardless of heightfield scale.
		 * 
		 * @param QueryBounds Bounding box in which we want to produce triangles, in HeightField space
		 * @param QueryTransform Transforms from HeightField space to query space (usually the other object's space)
		 * @param CullDistance The distance at which we can ignore triangles
		 * @param Visitor void(const FTriangle& Triangle, const int32 TriangleIndex, const int32 VertexIndex0, const int32 VertexIndex1, const int32 OutVertexIndex2)
		*/
		template<typename TriangleVisitor>
		void VisitTriangles(const FAABB3& QueryBounds, const FRigidTransform3& QueryTransform, const TriangleVisitor& Visitor) const
		{
			FBounds2D GridQueryBounds;
			GridQueryBounds.Min = FVec2(QueryBounds.Min()[0], QueryBounds.Min()[1]);
			GridQueryBounds.Max = FVec2(QueryBounds.Max()[0], QueryBounds.Max()[1]);

			// @todo(chaos): could we do the 3D bounds test here?
			TArray<TVec2<int32>> Intersections;
			GetGridIntersections(GridQueryBounds, Intersections);

			const bool bStandardWinding = ((GeomData.Scale.X * GeomData.Scale.Y * GeomData.Scale.Z) >= FReal(0));

			FVec3 Points[4];
			FTriangle Triangles[2];

			for (const TVec2<int32>& Cell : Intersections)
			{
				const int32 CellVertexIndex = Cell[1] * GeomData.NumCols + Cell[0];	// First vertex in cell
				const int32 CellIndex = Cell[1] * (GeomData.NumCols - 1) + Cell[0];

				// Check for holes and skip checking if we'll never collide
				if (GeomData.MaterialIndices.IsValidIndex(CellIndex) && GeomData.MaterialIndices[CellIndex] == TNumericLimits<uint8>::Max())
				{
					continue;
				}

				// The triangle is solid so proceed to test it
				FAABB3 CellBounds;
				GeomData.GetPointsAndBoundsScaled(CellVertexIndex, Points, CellBounds);

				if (CellBounds.Intersects(QueryBounds))
				{
					// Transform points into the space of the query
					// @todo(chaos): duplicate work here when we overlap lots of cells. We could generate the transformed verts for the full query once...
					Points[0] = QueryTransform.TransformPositionNoScale(Points[0]);
					Points[1] = QueryTransform.TransformPositionNoScale(Points[1]);
					Points[2] = QueryTransform.TransformPositionNoScale(Points[2]);
					Points[3] = QueryTransform.TransformPositionNoScale(Points[3]);

					// @todo(chaos): utility function for this
					const int32 VertexIndex0 = CellVertexIndex;
					const int32 VertexIndex1 = CellVertexIndex + 1;
					const int32 VertexIndex2 = CellVertexIndex + GeomData.NumCols;
					const int32 VertexIndex3 = CellVertexIndex + GeomData.NumCols + 1;

					// Generate contacts if overlapping
					const int32 FaceIndex0 = CellIndex * 2 + 0;
					const int32 FaceIndex1 = CellIndex * 2 + 1;

					if (bStandardWinding)
					{
						Triangles[0] = FTriangle(Points[0], Points[1], Points[3]);
						Triangles[1] = FTriangle(Points[0], Points[3], Points[2]);
						Visitor(Triangles[0], FaceIndex0, VertexIndex0, VertexIndex1, VertexIndex3);
						Visitor(Triangles[1], FaceIndex1, VertexIndex0, VertexIndex3, VertexIndex2);
					}
					else
					{
						Triangles[0] = FTriangle(Points[0], Points[3], Points[1]);
						Triangles[1] = FTriangle(Points[0], Points[2], Points[3]);
						Visitor(Triangles[0], FaceIndex0, VertexIndex0, VertexIndex3, VertexIndex1);
						Visitor(Triangles[1], FaceIndex1, VertexIndex0, VertexIndex2, VertexIndex3);
					}
				}
			}
		}

		// Internal: do not use - this API will change as we optimize mesh collision
		void CollectTriangles(const FAABB3& QueryBounds, const FRigidTransform3& QueryTransform, const FAABB3& ObjectBounds, Private::FMeshContactGenerator& Collector) const;

		virtual void VisitOverlappingLeafObjectsImpl(
			const FAABB3& QueryBounds,
			const FRigidTransform3& ObjectTransform,
			const int32 RootObjectIndex,
			int32& ObjectIndex,
			int32& LeafObjectIndex,
			const FImplicitHierarchyVisitor& VisitorFunc) const override final
		{
			if (IsOverlappingAnyCells(QueryBounds))
			{
				VisitorFunc(this, ObjectTransform, RootObjectIndex, ObjectIndex, LeafObjectIndex);
			}
			++ObjectIndex;
			++LeafObjectIndex;
		}

		virtual bool IsOverlappingBoundsImpl(const FAABB3& QueryBounds) const override final
		{
			return IsOverlappingAnyCells(QueryBounds);
		}

		// Does the mesh-space QueryBounds overlap the bounds of any cells in the heightfield?
		// @todo(chaos): we can return as soon as we overlap any cell so maybe use a 
		// custom function rather than GetBoundsScaled. Also we only care about Z overlap.
		bool IsOverlappingAnyCells(const FAABB3& QueryBounds) const
		{
			// Top-level bounds check
			if (!BoundingBox().Intersects(QueryBounds))
			{
				return false;
			}

			// Find all the cells that overlap in the X/Y plane
			FVec2 Scale2D(GeomData.Scale[0], GeomData.Scale[1]);
			const FBounds2D FlatBounds = GetFlatBounds();

			FBounds2D GridQueryBounds;
			GridQueryBounds.Min = FVec2(QueryBounds.Min()[0], QueryBounds.Min()[1]);
			GridQueryBounds.Max = FVec2(QueryBounds.Max()[0], QueryBounds.Max()[1]);
			GridQueryBounds = FBounds2D::FromPoints(FlatBounds.Clamp(GridQueryBounds.Min) / Scale2D, FlatBounds.Clamp(GridQueryBounds.Max) / Scale2D);

			FAABBVectorized QueryBoundsSimd = FAABBVectorized(QueryBounds);

			// We want to capture the first cell (delta == 0) as well
			TVec2<int32> FirstCell = FlatGrid.Cell(GridQueryBounds.Min);
			TVec2<int32> LastCell = FlatGrid.Cell(GridQueryBounds.Max);

			FAABBVectorized RegionBounds;
			GeomData.GetBoundsScaled(FirstCell, LastCell - FirstCell, RegionBounds);
			return RegionBounds.Intersects(QueryBoundsSimd);
		}

		struct FClosestFaceData
		{
			int32 FaceIndex = INDEX_NONE;
			FReal DistanceToFaceSq = TNumericLimits<FReal>::Max();
			bool bWasSampleBehind = false;
		};

		CHAOS_API FClosestFaceData FindClosestFace(const FVec3& Position, FReal SearchDist) const;
		
		virtual uint16 GetMaterialIndex(uint32 HintIndex) const override
		{
			ensure(GeomData.MaterialIndices.Num() > 0);

			// If we've only got a default
			if(GeomData.MaterialIndices.Num() == 1)
			{
				return GeomData.MaterialIndices[0];
			}
			else
			{
				// We store per cell for materials, so change to cell index
				int32 CellIndex = HintIndex / 2;
				if(GeomData.MaterialIndices.IsValidIndex(CellIndex))
				{
					return GeomData.MaterialIndices[CellIndex];
				}
			}
			
			// INDEX_NONE will be out of bounds but it is an expected value. If we reach this section of the code and the index isn't INDEX_NONE, we have an issue
			ensureMsgf(HintIndex == INDEX_NONE,TEXT("GetMaterialIndex called with an invalid MaterialIndex => %d"),HintIndex);
			
			return 0;
		}

		virtual const FAABB3 BoundingBox() const
		{
			// Generate a correct bound including scales (may be negative)
			CachedBounds = FAABB3::FromPoints(LocalBounds.Min() * GeomData.Scale, LocalBounds.Max() * GeomData.Scale);

			return CachedBounds;
		}

		virtual uint32 GetTypeHash() const override
		{
			TArray<uint8> Bytes;
			FMemoryWriter Writer(Bytes);
			FChaosArchive ChaosAr(Writer);

			// Saving to an archive is a const operation, but must be non-const
			// to support loading. Cast const away here to get bytes written
			const_cast<FHeightField*>(this)->Serialize(ChaosAr);

			return FCrc::MemCrc32(Bytes.GetData(), (int32)Bytes.GetAllocatedSize());
		}

		static constexpr EImplicitObjectType StaticType()
		{
			return ImplicitObjectType::HeightField;
		}

		virtual void Serialize(FChaosArchive& Ar) override
		{
			FChaosArchiveScopedMemory ScopedMemory(Ar, GetTypeName());
			FImplicitObject::SerializeImp(Ar);
			
			GeomData.Serialize(Ar);

			Ar.UsingCustomVersion(FExternalPhysicsCustomObjectVersion::GUID);
			if (Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) >= FExternalPhysicsCustomObjectVersion::HeightfieldData)
			{
				Ar << FlatGrid;
				Ar << FlattenedBounds.Min;
				Ar << FlattenedBounds.Max;
				TBox<FReal, 3>::SerializeAsAABB(Ar, LocalBounds);
			}
			else
			{
				CalcBounds();
			}
			

			if(Ar.IsLoading())
			{
				BuildQueryData();
				BoundingBox();	//temp hack to initialize cache
			}
		}

		void SetScale(const FVec3& InScale)
		{
			GeomData.Scale = InScale;
			GeomData.ScaleSimd = MakeVectorRegisterFloatFromDouble(MakeVectorRegisterDouble(InScale.X, InScale.Y, InScale.Z, 0.0));
		}

		template<typename InStorageType>
		struct FData
		{
			// For ease of access through typedefs
			using StorageType = InStorageType;

			// Only supporting unsigned int types for the height range - really no difference using
			// this or signed but this is a little nicer overall
			static_assert(std::is_same_v<StorageType, uint8> ||
				std::is_same_v<StorageType, uint16> ||
				std::is_same_v<StorageType, uint32> ||
				std::is_same_v<StorageType, uint64>,
				"Expected unsigned integer type for heightfield data storage");

			// Data sizes to validate during serialization
			static constexpr int32 RealSize = sizeof(FReal);
			static constexpr int32 StorageSize = sizeof(StorageType);

			// Range of the chosen type (unsigned so Min is always 0)
			static constexpr int32 StorageRange = TNumericLimits<StorageType>::Max();

			static constexpr int32 LowResInc = 6;

			// Heights in the chosen format. final placement of the vertex will be at
			// MinValue + Heights[Index] * HeightPerUnit
			// With HeightPerUnit being the range of the min/max FReal values of
			// the heightfield divided by the range of StorageType
			TArray<StorageType> Heights;

			struct MinMaxHeights
			{
				StorageType Min;
				StorageType Max;

				void Serialize(FChaosArchive& Ar)
				{
					Ar << Min;
					Ar << Max;
				}
			};
			TArray<MinMaxHeights> LowResolutionHeights;
			TArray<uint8> MaterialIndices;
			FVec3 Scale;
			VectorRegister4Float ScaleSimd;
			FReal MinValue;
			FReal MaxValue;
			uint16 NumRows;
			uint16 NumCols;
			FReal Range;
			FReal HeightPerUnit;
			uint16 NumColsLowRes;

			constexpr FReal GetCellWidth() const
			{
				return Scale[0];
			}

			constexpr FReal GetCellHeight() const
			{
				return Scale[1];
			}

			/**
			 * @brief Convert a Cell Index to a Vertex Index. 
			 * Returns the vertex index of the first corner in the cell. The other corners will be 
			 * (VertexIndex + 1), (VertexIndex + NumCols), (VertexIndex + NumCols + 1).
			 * Most functions in this class take a vertex index.
			 *
			 *	00 --- 01 --- 02 --- 03
			 *	|  0   |  1   |  2   |
			 *	04 --- 05 --- 06 --- 07
			 *	|  3   |  4   |  5   |
			 *	08 --- 09 --- 10 --- 11
			 *	|  6   |  7   |  8   |
			 *	12 --- 13 --- 14 --- 15
			 *
			*/
			FORCEINLINE int32 CellIndexToVertexIndex(const int32 CellIndex) const
			{
				const int32 CellY = CellIndex / (NumCols - 1);
				const int32 HeightIndex = CellIndex + CellY;
				return HeightIndex;
			}

			FORCEINLINE FVec3 GetPoint(int32 Index) const
			{
				const FReal Height = MinValue + Heights[Index] * HeightPerUnit;

				const int32 X = Index % (NumCols);
				const int32 Y = Index / (NumCols);

				return {(FReal)X, (FReal)Y, Height};
			}

			FORCEINLINE FVec3 GetPointScaled(int32 Index) const
			{
				return GetPoint(Index) * Scale;
			}

			FORCEINLINE void GetPoints(int32 Index, FVec3 OutPts[4]) const
			{
				const FReal H0 = MinValue + Heights[Index] * HeightPerUnit;
				const FReal H1 = MinValue + Heights[Index + 1] * HeightPerUnit;
				const FReal H2 = MinValue + Heights[Index + NumCols] * HeightPerUnit;
				const FReal H3 = MinValue + Heights[Index + NumCols + 1] * HeightPerUnit;

				const int32 X = Index % (NumCols);
				const int32 Y = Index / (NumCols);

				OutPts[0] = {(FReal)X, (FReal)Y, H0};
				OutPts[1] = {(FReal)X + 1, (FReal)Y, H1};
				OutPts[2] = {(FReal)X, (FReal)Y + 1, H2};
				OutPts[3] = {(FReal)X + 1, (FReal)Y + 1, H3};
			}

			FORCEINLINE void GetPointsAndBounds(int32 Index, FVec3 OutPts[4], FAABB3& OutBounds) const
			{
				const FReal H0 = MinValue + Heights[Index] * HeightPerUnit;
				const FReal H1 = MinValue + Heights[Index + 1] * HeightPerUnit;
				const FReal H2 = MinValue + Heights[Index + NumCols] * HeightPerUnit;
				const FReal H3 = MinValue + Heights[Index + NumCols + 1] * HeightPerUnit;

				const int32 X = Index % (NumCols);
				const int32 Y = Index / (NumCols);

				OutPts[0] = { (FReal)X, (FReal)Y, H0 };
				OutPts[1] = { (FReal)X + 1, (FReal)Y, H1 };
				OutPts[2] = { (FReal)X, (FReal)Y + 1, H2 };
				OutPts[3] = { (FReal)X + 1, (FReal)Y + 1, H3 };

				const FReal MinZ = FMath::Min<FReal>(H0, FMath::Min<FReal>(H1, FMath::Min<FReal>(H2, H3)));
				const FReal MaxZ = FMath::Max<FReal>(H0, FMath::Max<FReal>(H1, FMath::Max<FReal>(H2, H3)));

				OutBounds = FAABB3(FVec3((FReal)X, (FReal)Y, MinZ), FVec3((FReal)X + 1, (FReal)Y + 1, MaxZ));
			}

			FORCEINLINE void GetPointsAndBoundsSimd(int32 Index, VectorRegister4Float OutPts[4], FAABBVectorized& OutBounds) const
			{
				const FRealSingle H0 = static_cast<FRealSingle>(MinValue + Heights[Index] * HeightPerUnit);
				const FRealSingle H1 = static_cast<FRealSingle>(MinValue + Heights[Index + 1] * HeightPerUnit);
				const FRealSingle H2 = static_cast<FRealSingle>(MinValue + Heights[Index + NumCols] * HeightPerUnit);
				const FRealSingle H3 = static_cast<FRealSingle>(MinValue + Heights[Index + NumCols + 1] * HeightPerUnit);

				const int32 X = Index % (NumCols);
				const int32 Y = Index / (NumCols);

				OutPts[0] = MakeVectorRegisterFloat((FRealSingle)X, (FRealSingle)Y, H0, 0.0f );
				OutPts[1] = MakeVectorRegisterFloat((FRealSingle)X + 1, (FRealSingle)Y, H1, 0.0f);
				OutPts[2] = MakeVectorRegisterFloat((FRealSingle)X, (FRealSingle)Y + 1, H2, 0.0f);
				OutPts[3] = MakeVectorRegisterFloat((FRealSingle)X + 1, (FRealSingle)Y + 1, H3, 0.0f);

				const FRealSingle MinZ = FMath::Min<FRealSingle>(H0, FMath::Min<FRealSingle>(H1, FMath::Min<FRealSingle>(H2, H3)));
				const FRealSingle MaxZ = FMath::Max<FRealSingle>(H0, FMath::Max<FRealSingle>(H1, FMath::Max<FRealSingle>(H2, H3)));

				OutBounds = FAABBVectorized(MakeVectorRegisterFloat((FRealSingle)X, (FRealSingle)Y, MinZ, 0.0f),
											MakeVectorRegisterFloat((FRealSingle)X + 1, (FRealSingle)Y + 1, MaxZ, 0.0f));
			}

			FORCEINLINE void GetBoundsSimd(int32 Index, FAABBVectorized& OutBounds) const
			{
				const FRealSingle H0 = static_cast<FRealSingle>(MinValue + Heights[Index] * HeightPerUnit);
				const FRealSingle H1 = static_cast<FRealSingle>(MinValue + Heights[Index + 1] * HeightPerUnit);
				const FRealSingle H2 = static_cast<FRealSingle>(MinValue + Heights[Index + NumCols] * HeightPerUnit);
				const FRealSingle H3 = static_cast<FRealSingle>(MinValue + Heights[Index + NumCols + 1] * HeightPerUnit);

				const int32 X = Index % (NumCols);
				const int32 Y = Index / (NumCols);
				
				const FRealSingle MinZ = FMath::Min<FRealSingle>(H0, FMath::Min<FRealSingle>(H1, FMath::Min<FRealSingle>(H2, H3)));
				const FRealSingle MaxZ = FMath::Max<FRealSingle>(H0, FMath::Max<FRealSingle>(H1, FMath::Max<FRealSingle>(H2, H3)));

				OutBounds = FAABBVectorized(MakeVectorRegisterFloat((FRealSingle)X, (FRealSingle)Y, MinZ, 0.0f),
					MakeVectorRegisterFloat((FRealSingle)X + 1, (FRealSingle)Y + 1, MaxZ, 0.0f));
			}
			
			FORCEINLINE void GetBounds(TVec2<int32> CellIdx, TVec2<int32> Area, FAABBVectorized& OutBounds) const
			{
				FRealSingle MinHeight = UE_BIG_NUMBER;
				FRealSingle MaxHeight = -UE_BIG_NUMBER;

				const int32 EndIndexX = CellIdx[0] + Area[0];
				int32 FirstIndexX = FMath::Min<int32>(CellIdx[0], EndIndexX);
				FirstIndexX = FMath::Max<int32>(FirstIndexX, 0);
				int32 LastIndexX = FMath::Max<int32>(CellIdx[0], EndIndexX);
				// Increment LastIndex to look into the four corners of a cell 
				LastIndexX++;
				LastIndexX = FMath::Min<int32>(LastIndexX, NumCols-1);
				
				const int32 EndIndexY = CellIdx[1] + Area[1];
				int32 FirstIndexY = FMath::Min<int32>(CellIdx[1], EndIndexY);
				FirstIndexY = FMath::Max<int32>(FirstIndexY, 0);
				int32 LastIndexY = FMath::Max<int32>(CellIdx[1], EndIndexY);
				LastIndexY++;
				LastIndexY = FMath::Min<int32>(LastIndexY, NumRows-1);

				for (int IndexY = FirstIndexY; IndexY <= LastIndexY; IndexY++)
				{
					for (int IndexX = FirstIndexX; IndexX <= LastIndexX; IndexX++)
					{
						const int32 Index = IndexY * NumCols + IndexX;
						check(Index < Heights.Num());
						const FRealSingle CurrHeight = Heights[Index];
						MinHeight = FMath::Min<FRealSingle>(CurrHeight, MinHeight);
						MaxHeight = FMath::Max<FRealSingle>(CurrHeight, MaxHeight);
						
					}
				}

				const FRealSingle MinValueSingle = static_cast<FRealSingle>(MinValue);
				const FRealSingle HeightPerUnitSingle = static_cast<FRealSingle>(HeightPerUnit);
				MinHeight = MinValueSingle + MinHeight * HeightPerUnitSingle;
				MaxHeight = MinValueSingle + MaxHeight * HeightPerUnitSingle;

				OutBounds = FAABBVectorized(MakeVectorRegisterFloat(static_cast<FRealSingle>(FirstIndexX), static_cast<FRealSingle>(FirstIndexY), MinHeight, 0.0f),
											MakeVectorRegisterFloat(static_cast<FRealSingle>(LastIndexX), static_cast<FRealSingle>(LastIndexY), MaxHeight, 0.0f));
			}

			FORCEINLINE void GetLowResBounds(TVec2<int32> CellIdx, FAABBVectorized& OutBounds) const
			{
				const int32 IndexLowResX = CellIdx[0] / LowResInc;
				const int32 IndexLowResY = CellIdx[1] / LowResInc;

				const int32 Index = IndexLowResY * (NumColsLowRes - 1) + IndexLowResX + IndexLowResY;
				check(Index < LowResolutionHeights.Num());

				FRealSingle MinHeight = LowResolutionHeights[Index].Min;
				FRealSingle MaxHeight = LowResolutionHeights[Index].Max;

				const FRealSingle MinValueSingle = static_cast<FRealSingle>(MinValue);
				const FRealSingle HeightPerUnitSingle = static_cast<FRealSingle>(HeightPerUnit);
				MinHeight = MinValueSingle + MinHeight * HeightPerUnitSingle;
				MaxHeight = MinValueSingle + MaxHeight * HeightPerUnitSingle;

				OutBounds = FAABBVectorized(MakeVectorRegisterFloat(static_cast<FRealSingle>(IndexLowResX *LowResInc), static_cast<FRealSingle>(IndexLowResY *LowResInc), MinHeight, 0.0f),
					MakeVectorRegisterFloat(static_cast<FRealSingle>((IndexLowResX +1) * LowResInc), static_cast<FRealSingle>((IndexLowResY + 1) * LowResInc), MaxHeight, 0.0f));
			}

			FORCEINLINE void GetPointsScaled(int32 Index, FVec3 OutPts[4]) const
			{
				GetPoints(Index, OutPts);

				OutPts[0] *= Scale;
				OutPts[1] *= Scale;
				OutPts[2] *= Scale;
				OutPts[3] *= Scale;
			}

			FORCEINLINE void GetPointsAndBoundsScaled(int32 Index, FVec3 OutPts[4], FAABB3& OutBounds) const
			{
				GetPointsAndBounds(Index, OutPts, OutBounds);

				OutPts[0] *= Scale;
				OutPts[1] *= Scale;
				OutPts[2] *= Scale;
				OutPts[3] *= Scale;

				OutBounds = FAABB3::FromPoints(OutBounds.Min() * Scale, OutBounds.Max() * Scale);
			}

			FORCEINLINE void GetPointsAndBoundsScaledSimd(int32 Index, VectorRegister4Float OutPts[4], FAABBVectorized& OutBounds) const
			{
				GetPointsAndBoundsSimd(Index, OutPts, OutBounds);

				OutPts[0] = VectorMultiply(OutPts[0], ScaleSimd);
				OutPts[1] = VectorMultiply(OutPts[1], ScaleSimd);
				OutPts[2] = VectorMultiply(OutPts[2], ScaleSimd);
				OutPts[3] = VectorMultiply(OutPts[3], ScaleSimd);

				VectorRegister4Float P0 = VectorMultiply(OutBounds.GetMin(), ScaleSimd);
				VectorRegister4Float P1 = VectorMultiply(OutBounds.GetMax(), ScaleSimd);
				VectorRegister4Float Min = VectorMin(P0, P1);
				VectorRegister4Float Max = VectorMax(P0, P1);

				OutBounds = FAABBVectorized(Min, Max);
			}

			FORCEINLINE void GetBoundsScaledSimd(int32 Index, FAABBVectorized& OutBounds) const
			{
				GetBoundsSimd(Index, OutBounds);

				VectorRegister4Float P0 = VectorMultiply(OutBounds.GetMin(), ScaleSimd);
				VectorRegister4Float P1 = VectorMultiply(OutBounds.GetMax(), ScaleSimd);
				VectorRegister4Float Min = VectorMin(P0, P1);
				VectorRegister4Float Max = VectorMax(P0, P1);

				OutBounds = FAABBVectorized(Min, Max);
			}

			FORCEINLINE void GetBoundsScaled(TVec2<int32> CellIdx, TVec2<int32> Area, FAABBVectorized& OutBounds) const
			{
				GetBounds(CellIdx, Area, OutBounds);

				VectorRegister4Float P0 = VectorMultiply(OutBounds.GetMin(), ScaleSimd);
				VectorRegister4Float P1 = VectorMultiply(OutBounds.GetMax(), ScaleSimd);
				VectorRegister4Float Min = VectorMin(P0, P1);
				VectorRegister4Float Max = VectorMax(P0, P1);

				OutBounds = FAABBVectorized(Min, Max);
			}

			FORCEINLINE void GetBoundsScaled(TVec2<int32> CellIdx, FAABBVectorized& OutBounds) const
			{
				GetBounds(CellIdx, TVec2<int32>(1, 1), OutBounds);

				VectorRegister4Float P0 = VectorMultiply(OutBounds.GetMin(), ScaleSimd);
				VectorRegister4Float P1 = VectorMultiply(OutBounds.GetMax(), ScaleSimd);
				VectorRegister4Float Min = VectorMin(P0, P1);
				VectorRegister4Float Max = VectorMax(P0, P1);

				OutBounds = FAABBVectorized(Min, Max);
			}

			FORCEINLINE void GetLowResBoundsScaled(TVec2<int32> CellIdx, FAABBVectorized& OutBounds) const
			{
				GetLowResBounds(CellIdx, OutBounds);

				VectorRegister4Float P0 = VectorMultiply(OutBounds.GetMin(), ScaleSimd);
				VectorRegister4Float P1 = VectorMultiply(OutBounds.GetMax(), ScaleSimd);
				VectorRegister4Float Min = VectorMin(P0, P1);
				VectorRegister4Float Max = VectorMax(P0, P1);

				OutBounds = FAABBVectorized(Min, Max);
			}

			FORCEINLINE FReal GetMinHeight() const
			{
				return MinValue;
			}

			FORCEINLINE FReal GetMaxHeight() const
			{
				return MaxValue;
			}

			void Serialize(FChaosArchive& Ar)
			{
				// we need to account for the fact that FReal size may change
				const int32 RuntimeRealSize = RealSize;
				const int32 RunTimeStorageSize = StorageSize;


				int32 SerializedRealSize = RealSize;
				int32 SerializedStorageSize = StorageSize;

				Ar << SerializedRealSize;
				Ar << SerializedStorageSize;

				if(Ar.IsLoading())
				{
					// we only support float and double as FReal
					checkf(SerializedRealSize == sizeof(float) || SerializedRealSize == sizeof(double), TEXT("Heightfield was serialized with unexpected real type size (expected: 4 or 8, found: %d)"), SerializedRealSize);
					checkf(SerializedStorageSize == RunTimeStorageSize, TEXT("Heightfield was serialized with mismatched storage type size (expected: %d, found: %d)"), RunTimeStorageSize, SerializedStorageSize);
				}
				
				Ar << Heights;
				Ar << Scale;
				Ar << MinValue;
				Ar << MaxValue;
				Ar << NumRows;
				Ar << NumCols;

				ScaleSimd = MakeVectorRegisterFloatFromDouble(MakeVectorRegisterDouble(Scale.X, Scale.Y, Scale.Z, 0.0));

				Ar.UsingCustomVersion(FExternalPhysicsCustomObjectVersion::GUID);
				if (Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) >= FExternalPhysicsCustomObjectVersion::HeightfieldData)
				{
					Ar << Range;
					Ar << HeightPerUnit;

					if (Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) < FExternalPhysicsCustomObjectVersion::HeightfieldImplicitBounds)
					{
						// todo(chaos) this may not matter if the Vector types are handling serialization properly 
						// legacy, need to keep the inner box type as float ( not FReal ) 
						TArray<TUniqueObj<FBoxFloat3>> CellBounds;
						Ar << CellBounds;
					}
					else if(Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) < FExternalPhysicsCustomObjectVersion::HeightfieldUsesHeightsDirectly)
					{
						// legacy, need to keep the type as float ( not FReal ) 
						TArray<float> OldHeights;
						Ar << OldHeights;
					}
				}

				if(Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) >= FExternalPhysicsCustomObjectVersion::AddedMaterialManager)
				{
					Ar << MaterialIndices;
				}

				Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
				if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) == FUE5MainStreamObjectVersion::AddLowResolutionHeightField)
				{
					Ar << LowResolutionHeights;
					Ar << NumColsLowRes;
				}
				else if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) >= FUE5MainStreamObjectVersion::DecreaseLowResolutionHeightField)
				{
					Ar << LowResolutionHeights;
					Ar << NumColsLowRes;
				}
				else
				{
					BuildLowResolutionData();
				}
			}

			// A height bounds for grids of cells of size "LowResInc" square, for faster culling
			void BuildLowResolutionData()
			{
				const int32 NumRowsLowRes = FMath::CeilToInt(FRealSingle(NumRows) / LowResInc);
				NumColsLowRes = uint16(FMath::CeilToInt(FRealSingle(NumCols) / LowResInc));
				const int32 NumLowResHeights = NumRowsLowRes * NumColsLowRes;
				LowResolutionHeights.SetNum(NumLowResHeights);

				for (int32 RowIdxLowRes = 0; RowIdxLowRes < NumRowsLowRes; ++RowIdxLowRes)
				{
					for (int32 ColIdxLowRes = 0; ColIdxLowRes < NumColsLowRes; ++ColIdxLowRes)
					{
						FDataType::StorageType MinHeight = std::numeric_limits<FDataType::StorageType>::max();
						FDataType::StorageType MaxHeight = std::numeric_limits<FDataType::StorageType>::min();
						for (int32 RowIdx = RowIdxLowRes * int32(LowResInc); RowIdx < NumRows && RowIdx <= (RowIdxLowRes + 1) * int32(LowResInc); ++RowIdx)
						{
							for (int32 ColIdx = ColIdxLowRes * int32(LowResInc); ColIdx < NumCols && ColIdx <= (ColIdxLowRes + 1) * int32(LowResInc); ++ColIdx)
							{
								const int32 HeightIndex = RowIdx * NumCols + ColIdx;
								MaxHeight = FMath::Max<FDataType::StorageType>(Heights[HeightIndex], MaxHeight);
								MinHeight = FMath::Min<FDataType::StorageType>(Heights[HeightIndex], MinHeight);
							}
						}
						const int32 HeightLowResIndex = RowIdxLowRes * NumColsLowRes + ColIdxLowRes;
						LowResolutionHeights[HeightLowResIndex].Max = MaxHeight;
						LowResolutionHeights[HeightLowResIndex].Min = MinHeight;
					}
				}
			}
		};

		using FDataType = FData<uint16>;
		FDataType GeomData;

	private:

		// Struct for 2D bounds and associated operations
		struct FBounds2D
		{
			FVec2 Min;
			FVec2 Max;
			
			FBounds2D()
				: Min(0)
				, Max(0)
			{}

			FBounds2D(const FVec2& InMin, const FVec2& InMax)
				: Min(InMin)
				, Max(InMax)
			{}

			/**
			 * Given a set of points, wrap a 2D bounds around them
			 * @param P0 The first of the points to wrap
			 * @param InPoints Parameter pack of all subsequent points
			 */
			template<typename... Points>
			static FBounds2D FromPoints(const FVec2& P0, const Points&... InPoints)
			{
				static_assert(sizeof...(InPoints) > 0);
				static_assert(std::is_same_v<std::common_type_t<Points...>, FVec2>);

				FBounds2D Result(P0, P0);
				(Result.GrowToInclude(InPoints), ...);
				return Result;
			}

			explicit FBounds2D(const FAABB3& In3DBounds)
			{
				Set(In3DBounds);
			}

			void Set(const FAABB3& In3DBounds)
			{
				Min = {In3DBounds.Min()[0], In3DBounds.Min()[1]};
				Max = {In3DBounds.Max()[0], In3DBounds.Max()[1]};
			}

			FVec2 GetExtent() const
			{
				return Max - Min;
			}

			bool IsInside(const FVec2& InPoint) const
			{
				return InPoint[0] >= Min[0] && InPoint[0] <= Max[0] && InPoint[1] >= Min[1] && InPoint[1] <= Max[1];
			}

			void GrowToInclude(const FVec2& InPoint)
			{
				Min = { FMath::Min(Min.X, InPoint.X), FMath::Min(Min.Y, InPoint.Y) };
				Max = { FMath::Max(Max.X, InPoint.X), FMath::Max(Max.Y, InPoint.Y) };
			}

			void Inflate(const FVec2& InInflation)
			{
				Min -= InInflation;
				Max += InInflation;
			}

			FVec2 Clamp(const FVec2& InToClamp, FReal InNudge = UE_SMALL_NUMBER) const
			{
				const FVec2 NudgeVec(InNudge, InNudge);
				const FVec2 TestMin = Min + NudgeVec;
				const FVec2 TestMax = Max - NudgeVec;

				FVec2 OutVec = InToClamp;

				OutVec[0] = FMath::Max(OutVec[0], TestMin[0]);
				OutVec[1] = FMath::Max(OutVec[1], TestMin[1]);

				OutVec[0] = FMath::Min(OutVec[0], TestMax[0]);
				OutVec[1] = FMath::Min(OutVec[1], TestMax[1]);

				return OutVec;
			}

			bool IntersectLine(const FVec2& InStart, const FVec2& InEnd)
			{
				if(IsInside(InStart) || IsInside(InEnd))
				{
					return true;
				}

				const FVec2 Extent = GetExtent();
				FReal TA, TB;

				if(Utilities::IntersectLineSegments2D(InStart, InEnd, Min, FVec2(Min[0] + Extent[0], Min[1]), TA, TB)
					|| Utilities::IntersectLineSegments2D(InStart, InEnd, Min, FVec2(Min[0], Min[1] + Extent[1]), TA, TB)
					|| Utilities::IntersectLineSegments2D(InStart, InEnd, Max, FVec2(Max[0] - Extent[0], Max[1]), TA, TB)
					|| Utilities::IntersectLineSegments2D(InStart, InEnd, Max, FVec2(Max[0], Max[1] - Extent[1]), TA, TB))
				{
					return true;
				}

				return false;
			}

			bool ClipLine(const FVec3& InStart, const FVec3& InEnd, FVec2& OutClippedStart, FVec2& OutClippedEnd) const
			{
				FVec2 TempStart(InStart[0], InStart[1]);
				FVec2 TempEnd(InEnd[0], InEnd[1]);

				bool bLineIntersects = ClipLine(TempStart, TempEnd);

				OutClippedStart = TempStart;
				OutClippedEnd = TempEnd;

				return bLineIntersects;
			}

			bool ClipLine(FVec2& InOutStart, FVec2& InOutEnd) const
			{
				
				// Test we don't need to clip at all, quite likely with a heightfield so optimize for it.
				const bool bStartInside = IsInside(InOutStart);
				const bool bEndInside = IsInside(InOutEnd);
				if(bStartInside && bEndInside)
				{
					return true;
				}

				const FVec2 Dir = InOutEnd - InOutStart;

				// Tiny ray not inside so must be outside
				if(Dir.SizeSquared() < 1e-4)
				{
					return false;
				}

				bool bPerpendicular[2];
				FVec2 InvDir;
				for(int Axis = 0; Axis < 2; ++Axis)
				{
					bPerpendicular[Axis] = Dir[Axis] == 0;
					InvDir[Axis] = bPerpendicular[Axis] ? 0 : 1 / Dir[Axis];
				}

				

				if(bStartInside)
				{
					const FReal TimeToExit = ComputeTimeToExit(InOutStart,InvDir);
					InOutEnd = InOutStart + Dir * TimeToExit;
					return true;
				}

				if(bEndInside)
				{
					const FReal TimeToExit = ComputeTimeToExit(InOutEnd,-InvDir);
					InOutStart = InOutEnd - Dir * TimeToExit;
					return true;
				}

				//start and end outside, need to see if we even intersect
				FReal TimesToEnter[2] = {TNumericLimits<FReal>::Max(),TNumericLimits<FReal>::Max()};
				FReal TimesToExit[2] = {TNumericLimits<FReal>::Max(),TNumericLimits<FReal>::Max()};
				
				for(int Axis = 0; Axis < 2; ++Axis)
				{
					if(bPerpendicular[Axis])
					{
						if(InOutStart[Axis] >= Min[Axis] && InOutStart[Axis] <= Max[Axis])
						{
							TimesToEnter[Axis] = 0;
						}
					}
					else
					{
						if(Dir[Axis] > 0)
						{
							if(InOutStart[Axis] <= Max[Axis])
							{
								TimesToEnter[Axis] = FMath::Max<FReal>(Min[Axis] - InOutStart[Axis], 0) * InvDir[Axis];
								TimesToExit[Axis] = (Max[Axis] - InOutStart[Axis])  * InvDir[Axis];
							}
						}
						else if(Dir[Axis] < 0)
						{
							if(InOutStart[Axis] >= Min[Axis])
							{
								TimesToEnter[Axis] = FMath::Max<FReal>(InOutStart[Axis] - Max[Axis],0) * InvDir[Axis];
								TimesToExit[Axis] = (InOutStart[Axis] - Min[Axis]) * InvDir[Axis];
							}
						}
					}
				}

				const FReal TimeToEnter = FMath::Max(FMath::Abs(TimesToEnter[0]),FMath::Abs(TimesToEnter[1]));
				const FReal TimeToExit = FMath::Min(FMath::Abs(TimesToExit[0]),FMath::Abs(TimesToExit[1]));

				if(TimeToExit < TimeToEnter)
				{
					//no intersection
					return false;
				}

				InOutEnd = InOutStart + Dir * TimeToExit;
				InOutStart = InOutStart + Dir * TimeToEnter;
				return true;
			}

		private:
			//This helper assumes Start is inside the min/max box and uses InvDir to compute how long it takes to exit
			FReal ComputeTimeToExit(const FVec2& Start,const FVec2& InvDir) const
			{
				FReal Times[2] ={TNumericLimits<FReal>::Max(),TNumericLimits<FReal>::Max()};
				for(int Axis = 0; Axis < 2; ++Axis)
				{
					if(InvDir[Axis] > 0)
					{
						Times[Axis] = (Max[Axis] - Start[Axis]) * InvDir[Axis];
					}
					else if(InvDir[Axis] < 0)
					{
						Times[Axis] = (Start[Axis] - Min[Axis]) * InvDir[Axis];
					}
				}

				const FReal MinTime = FMath::Min(FMath::Abs(Times[0]),FMath::Abs(Times[1]));
				return MinTime;
			}
		};

		// Helpers for accessing bounds
		CHAOS_API bool GetCellBounds2D(const TVec2<int32> InCoord, FBounds2D& OutBounds, const FVec2& InInflate = {0}) const;
		CHAOS_API bool GetCellBounds3D(const TVec2<int32> InCoord, FVec3& OutMin, FVec3& OutMax, const FVec3& InInflate = FVec3(0)) const;
		CHAOS_API bool GetCellBounds2DScaled(const TVec2<int32> InCoord, FBounds2D& OutBounds, const FVec2& InInflate = {0}) const;
		CHAOS_API bool GetCellBounds3DScaled(const TVec2<int32> InCoord, FVec3& OutMin, FVec3& OutMax, const FVec3& InInflate = FVec3(0)) const;
		CHAOS_API bool CalcCellBounds3D(const TVec2<int32> InCoord, FVec3& OutMin, FVec3& OutMax, const FVec3& InInflate = FVec3(0)) const;

		// Query functions - sweep, ray, overlap
		template<typename SQVisitor>
		bool GridSweep(const FVec3& StartPoint, const FVec3& Dir, const FReal Length, const FVec3 InHalfExtents, SQVisitor& Visitor) const;
		
		struct FWalkingData {

			explicit FWalkingData(FHeightfieldRaycastVisitor& VisitorIn) : Visitor(VisitorIn) {}
			VectorRegister4Float CurrentLengthSimd;
			FVec2 ScaledMin;
			FReal CurrentLength;
			FReal ZMidPoint;
			FVec3 Dir;
			FVec3 InvDir; 
			FVec3 NextStart;
			FVec3 ScaleSign;
			FVec3 ScaledDx;
			FVec2 ScaledDx2D;
			TVec2<int32> CellIdx;
			bool bParallel[3];
			FHeightfieldRaycastVisitor& Visitor;
		};

		CHAOS_API bool WalkSlow(FWalkingData& WalkingData, int32 IndexLowResX, int32 IndexLowResY) const;
		CHAOS_API bool WalkFast(FWalkingData& WalkingData, const FVec2& Scale2D, const FVec3& DirScaled) const;
		CHAOS_API bool WalkOnLowRes(FWalkingData& WalkingData, const FVec2& Scale2D, const FVec3& DirScaled) const;

		CHAOS_API bool GridCast(const FVec3& StartPoint, const FVec3& Dir, const FReal Length, FHeightfieldRaycastVisitor& Visitor) const;
		CHAOS_API bool GetGridIntersections(FBounds2D InFlatBounds, TArray<TVec2<int32>>& OutInterssctions) const;
		CHAOS_API bool GetGridIntersectionsBatch(FBounds2D InFlatBounds, TArray<TVec2<int32>>& OutIntersections, const FAABBVectorized& Bounds) const;
		
		// Get the cell range that overlaps the QueryBounds (local space). NOTE: OutEndCell is like an end iterator (i.e., BeginCell==EndCell is an empty range)
		// NOTE: This is a 2D overlap, ignoring height
		CHAOS_API bool GetOverlappingCellRange(const FAABB3& QueryBounds, TVec2<int32>& OutBeginCell, TVec2<int32>& OutEndCell) const;

		CHAOS_API FBounds2D GetFlatBounds() const;

		// Grid for queries, faster than bounding volumes for heightfields
		TUniformGrid<FReal, 2> FlatGrid;
		// Bounds in 2D of the whole heightfield, to clip queries against
		FBounds2D FlattenedBounds;
		// 3D bounds for the heightfield, for insertion to the scene structure
		FAABB3 LocalBounds;
		// Cached when bounds are requested. Mutable to allow GetBounds to be logical const
		mutable FAABB3 CachedBounds;

		CHAOS_API void CalcBounds();
		CHAOS_API void BuildQueryData();

		// Needed for serialization
		FHeightField() : FImplicitObject(EImplicitObject::HasBoundingBox, ImplicitObjectType::HeightField) {}
		friend FImplicitObject;
		friend UE::Physics::FHeightFieldGeometry;

		template <typename QueryGeomType>
		bool OverlapGeomImp(const QueryGeomType& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD = nullptr) const;

		template <typename QueryGeomType>
		bool SweepGeomImp(const QueryGeomType& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, const FReal Thickness, bool bComputeMTD) const;

		template <typename QueryGeomType>
		bool SweepGeomCCDImp(const QueryGeomType& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& OutTOI, FReal& OutPhi, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex) const;

		template <typename GeomType>
		bool GJKContactPointImp(const GeomType& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FVec3& ContactLocation, FVec3& ContactNormal, FReal& ContactPhi, int32& ContactFaceIndex) const;
	};

	FORCEINLINE FChaosArchive& operator<<(FChaosArchive& Ar, FHeightField::FDataType::MinMaxHeights& Value)
	{
		Value.Serialize(Ar);
		return Ar;
	}
}
