// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Box.h"
#include "Chaos/Vector.h"
#include "Containers/Map.h"
#include "Chaos/Framework/Parallel.h"
#include "Math/VectorRegister.h"

#include <atomic>

namespace Chaos
{
template<typename PayloadType>
class TSpatialHashGridBase
{
private:	
	struct FPayloadAndBounds
	{
		PayloadType Payload;
		int32 BoundsLookupIdx;
	};

	struct FHashIndex
	{
		VectorRegister4Int VectorIndex;

		FHashIndex() = default;

		FHashIndex(const TVec3<int32>& Cell, int32 Lod)
			:VectorIndex(MakeVectorRegisterInt(Cell.X, Cell.Y, Cell.Z, Lod))
		{}
		FHashIndex(int32 CellX, int32 CellY, int32 CellZ, int32 Lod)
			:VectorIndex(MakeVectorRegisterInt(CellX, CellY, CellZ, Lod))
		{}
		FHashIndex(const VectorRegister4Int& InVectorIndex)
			:VectorIndex(InVectorIndex)
		{}

		bool operator==(const FHashIndex& Other) const
		{
			union
			{
				VectorRegister4Float v;
				VectorRegister4Int i;
			}Result;
			Result.i = VectorIntCompareNEQ(VectorIndex, Other.VectorIndex);
			return VectorMaskBits(Result.v) == 0;
		}

		inline uint32 GetTypeHash() const
		{
			// xorHash
			static constexpr VectorRegister4Int Multiplier = MakeVectorRegisterIntConstant(73856093, 19349663, 83492791, 67867979);
			const VectorRegister4Int Multiplied = VectorIntMultiply(VectorIndex, Multiplier);
			alignas(VectorRegister4Int) int32 Indexable[4];
			VectorIntStoreAligned(Multiplied, Indexable);
			return (uint32)(Indexable[0] ^ Indexable[1] ^ Indexable[2] ^ Indexable[3]);
		}
	};

	// This hashmap is designed to be fast at just what we need to do to build all at once in exactly the order it is done in Initialize and then query. 
	// Based off of Core/HashTable.h
	class FHashMap
	{
	public:
		void Reset()
		{
			Hash.Reset();
			Elements.Reset();
		}

		void PreallocateElementsForConcurrentAdd(int32 MaxNumElements, int32 HashSize)
		{
			HashSize = FMath::RoundUpToPowerOfTwo(HashSize);
			Hash.SetNumUninitialized(HashSize);
			for (std::atomic<int32>& H : Hash)
			{
				H.store(INDEX_NONE, std::memory_order_relaxed);
			}
			Elements.SetNumUninitialized(MaxNumElements);
			ConcurrentElementAddIdx.store(0);
		}

		void ConcurrentAddElementRange(const TVec3<int32>& MinCellIdx, const TVec3<int32>& MaxCellIdx, int32 Lod, const FPayloadAndBounds& Value)
		{
			const int32 NumElementsToAdd = (MaxCellIdx[0] - MinCellIdx[0] + 1) * (MaxCellIdx[1] - MinCellIdx[1] + 1) * (MaxCellIdx[2] - MinCellIdx[2] + 1);

			if (ensureAlways(NumElementsToAdd <= 8))
			{
				int32 IndexToWrite = ConcurrentElementAddIdx.fetch_add(NumElementsToAdd, std::memory_order_relaxed);

				for (int32 XIdx = MinCellIdx[0]; XIdx <= MaxCellIdx[0]; ++XIdx)
				{
					for (int32 YIdx = MinCellIdx[1]; YIdx <= MaxCellIdx[1]; ++YIdx)
					{
						for (int32 ZIdx = MinCellIdx[2]; ZIdx <= MaxCellIdx[2]; ++ZIdx)
						{
							FElement Element;
							Element.Key = FHashIndex(XIdx, YIdx, ZIdx, Lod);
							const uint32 HashKey = Element.Key.GetTypeHash() & (Hash.Num() - 1);
							Element.NextIndex = Hash[HashKey].load(std::memory_order_relaxed);
							while (!Hash[HashKey].compare_exchange_weak(Element.NextIndex, IndexToWrite, std::memory_order_release, std::memory_order_relaxed));
							Element.Value = Value;
							Elements[IndexToWrite] = MoveTemp(Element);
							++IndexToWrite;
						}
					}
				}
			}
		}

		int32 ConcurrentAddElement(const TVec3<int32>& CellIdx, int32 Lod, const FPayloadAndBounds& Value)
		{
			const int32 IndexToWrite = ConcurrentElementAddIdx.fetch_add(1, std::memory_order_relaxed);
			FElement Element;
			Element.Key = FHashIndex( CellIdx, Lod );
			const uint32 HashKey = Element.Key.GetTypeHash() & (Hash.Num() - 1);
			Element.NextIndex = Hash[HashKey].load(std::memory_order_relaxed);
			while (!Hash[HashKey].compare_exchange_weak(Element.NextIndex, IndexToWrite, std::memory_order_release, std::memory_order_relaxed));
			Element.Value = Value;
			Elements[IndexToWrite] = MoveTemp(Element);

			return IndexToWrite;
		}

		void ShrinkElementsAfterConcurrentAdd()
		{
			Elements.SetNum(ConcurrentElementAddIdx.load(), EAllowShrinking::No);
		}

		int32 First(const FHashIndex& Key) const
		{
			checkSlow(FMath::IsPowerOfTwo(Hash.Num()));
			const uint32 HashKey = Key.GetTypeHash() &(Hash.Num() - 1);
			return Hash[HashKey].load(std::memory_order_relaxed);
		}

		int32 Next(int32 Index) const
		{
			return Elements[Index].NextIndex;
		}

		static bool IsValid(int32 Index)
		{
			return Index != INDEX_NONE;
		}

		bool KeyMatches(int32 Index, const FHashIndex& Key) const
		{
			return Elements[Index].Key == Key;
		}

		const FPayloadAndBounds& GetValue(int32 Index) const
		{
			return Elements[Index].Value;
		}

		const FHashIndex& GetKey(int32 Index) const
		{
			return Elements[Index].Key;
		}

	private:

		TArray<std::atomic<int32>> Hash;
		struct FElement
		{
			FHashIndex Key;
			FPayloadAndBounds Value;
			int32 NextIndex;
		};

		TArray<FElement> Elements;
		std::atomic<int32> ConcurrentElementAddIdx;
	};

	FHashMap HashMap;

	void Reset()
	{
		HashMap.Reset();
	}

	static inline TVec3<int32> CellIdxForPoint(const VectorRegister4Float& Point, const VectorRegister4Float& CellSize)
	{
		VectorRegister4Float Tmp = VectorDivide(Point, CellSize);

		VectorRegister4Int CellVector = VectorRoundToIntHalfToEven(VectorAdd(Tmp, VectorSubtract(Tmp, GlobalVectorConstants::FloatOneHalf)));
		CellVector = VectorShiftRightImmArithmetic(CellVector, 1);
		TVec4<int32> Result4;
		VectorIntStore(CellVector, &Result4[0]);
		return TVec3<int32>(Result4[0], Result4[1], Result4[2]);
	}

	static inline TVec3<int32> CellIdxForPoint(const VectorRegister4Double& Point, const VectorRegister4Double& CellSize)
	{
		VectorRegister4Double Tmp = VectorDivide(Point, CellSize);

		VectorRegister4Int CellVector = VectorRoundToIntHalfToEven(MakeVectorRegisterFloatFromDouble(VectorAdd(Tmp, VectorSubtract(Tmp, GlobalVectorConstants::DoubleOneHalf))));
		CellVector = VectorShiftRightImmArithmetic(CellVector, 1);
		TVec4<int32> Result4;
		VectorIntStore(CellVector, &Result4[0]);
		return TVec3<int32>(Result4[0], Result4[1], Result4[2]);
	}

	template<typename PT, typename T> friend class THierarchicalSpatialHash;
	template<typename PT, typename T> friend class TSpatialHashGridPoints;
};

// Currently this assumes a 3 dimensional grid. 
template<typename PayloadType, typename T>
class THierarchicalSpatialHash : public TSpatialHashGridBase<PayloadType>
{
	typedef TSpatialHashGridBase<PayloadType> Base;
	using typename Base::FPayloadAndBounds;
	using typename Base::FHashMap;
	using typename Base::FHashIndex;
	using Base::HashMap;

	static int32 LodForCellSize(T CellSize)
	{
		// want Ceil( log2(CellSize) )
		if (CellSize <= (T).5)
		{
			return -(int32)FMath::FloorLog2((uint32)FMath::FloorToInt32((T)1 / CellSize));
		}
		else
		{
			return (int32)FMath::CeilLogTwo((uint32)FMath::CeilToInt32(CellSize));
		}
	}

	static T CellSizeForLod(const int32 Lod)
	{
		if (Lod >= 0)
		{
			return (T)(1ULL << Lod);
		}
		return (T)1 / (T)(1ULL << (-Lod));
	}

public:

	class FVectorAABB
	{
	public:

		FVectorAABB() = default;

		explicit FVectorAABB(const TAABB<T, 3>& BBox)
			: Min(MakeVectorRegister(BBox.Min()[0], BBox.Min()[1], BBox.Min()[2], (T)0.))
			, Max(MakeVectorRegister(BBox.Max()[0], BBox.Max()[1], BBox.Max()[2], (T)0.))
		{}

		explicit FVectorAABB(const TVec3<T>& Point)
			: Min(MakeVectorRegister(Point[0], Point[1], Point[2], (T)0.))
			, Max(Min)
		{}

		FVectorAABB(const TVec3<T>& Point, const T Radius)
			: Min(MakeVectorRegister(Point[0], Point[1], Point[2], (T)0.))
			, Max(Min)
		{
			Thicken(Radius);
		}

		void GrowToInclude(const TVec3<T>& Point)
		{
			const TVectorRegisterType<T> Vector = MakeVectorRegister(Point[0], Point[1], Point[2], (T)0);
			Min = VectorMin(Min, Vector);
			Max = VectorMax(Max, Vector);
		}

		void GrowToInclude(const TVec3<T>& Point, const T Radius)
		{
			const TVectorRegisterType<T> Vector = MakeVectorRegister(Point[0], Point[1], Point[2], (T)0);
			const TVectorRegisterType<T> ThicknessVec = MakeVectorRegister(Radius, Radius, Radius, (T)0);
			Min = VectorMin(Min, VectorSubtract(Vector, ThicknessVec));
			Max = VectorMax(Max, VectorAdd(Vector, ThicknessVec));
		}

		void Thicken(const T Thickness)
		{
			const TVectorRegisterType<T> ThicknessVec = MakeVectorRegister(Thickness, Thickness, Thickness, (T)0);
			Min = VectorSubtract(Min, ThicknessVec);
			Max = VectorAdd(Max, ThicknessVec);
		}

		T LongestSideLength() const
		{
			const TVectorRegisterType<T> ExtentsVector = VectorSubtract(Max, Min);
			TVec3<T> Extents;
			VectorStoreFloat3(ExtentsVector, &Extents);
			return Extents.Max();
		}

		const TVectorRegisterType<T>& GetMin() const { return Min; }
		const TVectorRegisterType<T>& GetMax() const { return Max; }

		bool Intersects(const FVectorAABB& Other) const
		{
			if (VectorAnyLesserThan(Other.Max, Min) || VectorAnyGreaterThan(Other.Min, Max))
			{
				return false;
			}
			return true;
		}

		bool Intersects(const TVectorRegisterType<T>& Point) const
		{
			if (VectorAnyLesserThan(Point, Min) || VectorAnyGreaterThan(Point, Max))
			{
				return false;
			}
			return true;
		}

	private:

		TVectorRegisterType<T> Min;
		TVectorRegisterType<T> Max;
	};

	void Reset()
	{
		Base::Reset();
		UsedLods.Reset();
	}

	template<typename ParticlesType>
	void Initialize(const ParticlesType& Particles, const T MinLodSize = (T)0)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(HierarchicalSpatialHash_Init);
		Reset();

		Bounds.SetNumUninitialized(Particles.Num());
		HashMap.PreallocateElementsForConcurrentAdd(Particles.Num() * 8, Particles.Num() * 2);

		const int32 MinAllowableLod = MinLodSize > (T)0 ? LodForCellSize(MinLodSize) : std::numeric_limits<int32>::min();

		// For most common UsedLods, store whether or not the Lod is used in an array without a lock.
		// For uncommon Lods, use a critical section to store directly into the UsedLods map.
		constexpr int32 MaxPreAllocatedUsedLodValue = 10; // i.e., a single element 2^10 = 1024 across
		constexpr int32 MinPreAllocatedUsedLodValue = -10; // i.e., a single element 1 / (2^10) ~= 0.001 across.
		TStaticArray<bool, MaxPreAllocatedUsedLodValue - MinPreAllocatedUsedLodValue + 1> UsedLodsArray;
		memset(UsedLodsArray.GetData(), 0, UsedLodsArray.Num() * sizeof(bool));
		FCriticalSection UsedLodsCriticalSection;

		PhysicsParallelFor(Particles.Num(),
			[this, &Particles, MinAllowableLod, MaxPreAllocatedUsedLodValue, MinPreAllocatedUsedLodValue, &UsedLodsArray, &UsedLodsCriticalSection](int32 ParticleIdx)
			{
				const auto& Particle = Particles[ParticleIdx];
				const FVectorAABB ParticleBounds = Particle.VectorAABB();

				const int32 Lod = FMath::Max(MinAllowableLod, LodForCellSize(ParticleBounds.LongestSideLength()));
				const T CellSize = CellSizeForLod(Lod);
				const TVectorRegisterType<T> CellSizeVector = MakeVectorRegister(CellSize, CellSize, CellSize, CellSize);

				const PayloadType Payload = Particle.template GetPayload<PayloadType>(ParticleIdx);

				const TVec3<int32> MinCellIdx = Base::CellIdxForPoint(ParticleBounds.GetMin(), CellSizeVector);
				const TVec3<int32> MaxCellIdx = Base::CellIdxForPoint(ParticleBounds.GetMax(), CellSizeVector);

				const FPayloadAndBounds Value = { Payload, ParticleIdx };
				HashMap.ConcurrentAddElementRange(MinCellIdx, MaxCellIdx, Lod, Value);

				Bounds[ParticleIdx] = ParticleBounds;
				if (Lod >= MinPreAllocatedUsedLodValue && Lod <= MaxPreAllocatedUsedLodValue)
				{
					UsedLodsArray[Lod - MinPreAllocatedUsedLodValue] = true;
				}
				else
				{
					UsedLodsCriticalSection.Lock();
					UsedLods.Add(Lod, CellSize);
					UsedLodsCriticalSection.Unlock();
				}
			}
		);

		HashMap.ShrinkElementsAfterConcurrentAdd();

		for (int32 Lod = MinPreAllocatedUsedLodValue; Lod <= MaxPreAllocatedUsedLodValue; ++Lod)
		{
			if (UsedLodsArray[Lod - MinPreAllocatedUsedLodValue])
			{
				UsedLods.Add(Lod, CellSizeForLod(Lod));
			}
		}
	}	
	
	TArray<PayloadType> FindAllIntersections(const FVectorAABB LookupBounds,
		TFunctionRef<bool(const int32 Payload)> BroadphaseTest) const
	{
		// LookupBounds intentionally passed by value--it seems faster if it gets copied locally.
		TArray<PayloadType> Result;
		for (const typename TMap<int32, T>::ElementType& UsedLod : UsedLods)
		{
			const int32 Lod = UsedLod.Key;
			const TVectorRegisterType<T> CellSize = MakeVectorRegister(UsedLod.Value, UsedLod.Value, UsedLod.Value, UsedLod.Value);

			const TVec3<int32> MinCellIdx = Base::CellIdxForPoint(LookupBounds.GetMin(), CellSize);
			const TVec3<int32> MaxCellIdx = Base::CellIdxForPoint(LookupBounds.GetMax(), CellSize);

			for (int32 XIdx = MinCellIdx[0]; XIdx <= MaxCellIdx[0]; ++XIdx)
			{
				for (int32 YIdx = MinCellIdx[1]; YIdx <= MaxCellIdx[1]; ++YIdx)
				{
					for (int32 ZIdx = MinCellIdx[2]; ZIdx <= MaxCellIdx[2]; ++ZIdx)
					{
						const FHashIndex Key{ {XIdx, YIdx, ZIdx}, Lod };
						for (int32 HashIndex = HashMap.First(Key); HashMap.IsValid(HashIndex); HashIndex = HashMap.Next(HashIndex))
						{
							if (HashMap.KeyMatches(HashIndex, Key))
							{
								const FPayloadAndBounds& Value = HashMap.GetValue(HashIndex);
								if (!BroadphaseTest(Value.Payload))
								{
									continue;
								}

								const FVectorAABB& PayloadBounds = Bounds[Value.BoundsLookupIdx];

								if (PayloadBounds.Intersects(LookupBounds))
								{
									// Typical cloth examples do not find a lot of unique results. 
									// In testing, the overhead to do this search is lower than using a TSet and converting the result to an array
									Result.AddUnique(Value.Payload);
								}
							}
						}
					}
				}
			}
		}
		return Result;
	}

	TArray<PayloadType> FindAllIntersections(const TAABB<T, 3>& LookupBounds) const
	{
		return FindAllIntersections(FVectorAABB(LookupBounds), [](const int32) { return true; });
	}

	TArray<PayloadType> FindAllIntersections(const TVec3<T>& Point) const
	{
		TVectorRegisterType<T> PointVec(MakeVectorRegister(Point.X, Point.Y, Point.Z, (T)0));

		TArray<PayloadType> Result;
		for (const typename TMap<int32, T>::ElementType& UsedLod : UsedLods)
		{
			const int32 Lod = UsedLod.Key;
			const TVectorRegisterType<T> CellSize = MakeVectorRegister(UsedLod.Value, UsedLod.Value, UsedLod.Value, UsedLod.Value);

			const TVec3<int32> CellIdx = Base::CellIdxForPoint(PointVec, CellSize);
			const FHashIndex Key{ CellIdx, Lod };
			for (int32 HashIndex = HashMap.First(Key); HashMap.IsValid(HashIndex); HashIndex = HashMap.Next(HashIndex))
			{
				if (HashMap.KeyMatches(HashIndex, Key))
				{
					const FPayloadAndBounds Value = HashMap.GetValue(HashIndex);
					const FVectorAABB& PayloadBounds = Bounds[Value.BoundsLookupIdx];

					if (PayloadBounds.Intersects(PointVec))
					{
						// Typical cloth examples do not find a lot of unique results. 
						// In testing, the overhead to do this search is lower than using a TSet and converting the result to an array
						Result.AddUnique(Value.Payload);
					}
				}
			}
		}
		return Result;
	}

private:

	TMap<int32, T> UsedLods;
	TArray<FVectorAABB> Bounds;
};

template<typename PayloadType, typename T>
class TSpatialHashGridPoints : public TSpatialHashGridBase<PayloadType>
{
	typedef TSpatialHashGridBase<PayloadType> Base;
	using typename Base::FPayloadAndBounds;
	using typename Base::FHashMap;
	using typename Base::FHashIndex;
	using Base::HashMap;

public:
	TSpatialHashGridPoints(const T InCellSize)
		: Base()
		, CellSize(MakeVectorRegister(InCellSize, InCellSize, InCellSize, InCellSize))
	{}

	void Reset()
	{
		Base::Reset();
		ParticleIndexToElement.Reset();
	}

	template<typename ParticlesType>
	void InitializePoints(const ParticlesType& Particles, bool bForceSingleThreaded = false)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SpatialHashGridConstantBounds_Init);
		Reset();

		ParticleIndexToElement.SetNum(Particles.Num());
		HashMap.PreallocateElementsForConcurrentAdd(Particles.Num(), Particles.Num());
		PhysicsParallelFor(Particles.Num(),
			[this, &Particles](int32 ParticleIdx)
		{
			const auto& Particle = Particles[ParticleIdx];
			const PayloadType Payload = Particle.template GetPayload<PayloadType>(ParticleIdx);
			const auto& ParticleX = Particle.X();
			const TVec3<int32> CellIdx = Base::CellIdxForPoint(MakeVectorRegister(ParticleX[0], ParticleX[1], ParticleX[2], 0), CellSize);
			const FPayloadAndBounds Value = { Payload, ParticleIdx };
			const int32 ElementIdx = HashMap.ConcurrentAddElement(CellIdx, INDEX_NONE, Value);
			ParticleIndexToElement[ParticleIdx] = ElementIdx;
		}, bForceSingleThreaded
		);

		HashMap.ShrinkElementsAfterConcurrentAdd();
	}

	TArray<TVec2<PayloadType>> FindAllSelfProximities(int32 CellRadius, int32 MaxNumExpectedConnections, TFunctionRef<bool(const int32 Payload0, const int32 Payload1)> NarrowTest, bool bForceSingleThreaded = false) const
	{
		check(CellRadius >= 0);

		// Working around MSVC lambda bug with this functor.
		struct FParticleProximities
		{
			const int32 CellRadius;
			TFunctionRef<bool(const int32 Payload0, const int32 Payload1)> NarrowTest;
			const FHashMap& HashMap;
			const TArray<int32>& ParticleIndexToElement;

			// Resize using a RWLock. Start with MaxNumExpectedConnections
			TArray<TVec2<PayloadType>> Result;
			std::atomic<int32> ResultIndex;
			std::atomic<int32> CurrentResultNum; // Arrays aren't thread-safe, so use this to track size instead.
			FRWLock ResultResizeRWLock;

			FParticleProximities(const int32 InCellRadius, const int32 InMaxNumExpectedConnections,
				TFunctionRef<bool(const int32 Payload0, const int32 Payload1)> InNarrowTest,
				const FHashMap& InHashMap, const TArray<int32>& InParticleIndexToElement)
				:CellRadius(InCellRadius), NarrowTest(InNarrowTest), HashMap(InHashMap), ParticleIndexToElement(InParticleIndexToElement), ResultIndex(0)
			{
				Result.SetNum(InMaxNumExpectedConnections);
				Result.SetNum(Result.Max()); // Remove all slack.
				CurrentResultNum.store(Result.Num());
			}

			void AddResult(const TVec2<PayloadType>& Proximity)
			{
				if (!NarrowTest(Proximity[0], Proximity[1]))
				{
					return;
				}
				const int32 NewResultIndex = ResultIndex.fetch_add(1);
				if (NewResultIndex < CurrentResultNum.load())
				{
					ResultResizeRWLock.ReadLock();
					Result[NewResultIndex] = Proximity;
					ResultResizeRWLock.ReadUnlock();
				}
				else
				{
					ResultResizeRWLock.WriteLock();
					if (NewResultIndex >= CurrentResultNum.load())
					{
						// Use array's allocator to decide how much slack to create
						Result.SetNum(NewResultIndex + 1);
						Result.SetNum(Result.Max());
						CurrentResultNum.store(Result.Num());
					}
					Result[NewResultIndex] = Proximity;
					ResultResizeRWLock.WriteUnlock();
				}
			}

			void AddResultsInCell(PayloadType ThisPayload, const FHashIndex& Cell)
			{
				for (int32 HashIndex = HashMap.First(Cell); HashMap.IsValid(HashIndex); HashIndex = HashMap.Next(HashIndex))
				{
					if (HashMap.KeyMatches(HashIndex, Cell))
					{
						AddResult(TVec2<PayloadType>(ThisPayload, HashMap.GetValue(HashIndex).Payload));
					}
				}
			}

			void operator()(int32 ParticleIdx)
			{
				const int32 ThisElementIdx = ParticleIndexToElement[ParticleIdx];
				const int32 ThisPayload = HashMap.GetValue(ThisElementIdx).Payload;
				const FHashIndex& ThisKey = HashMap.GetKey(ThisElementIdx);

				// All neighbors in our cell after us in the list
				int32 OtherElementIdx = HashMap.Next(ThisElementIdx);
				for (; HashMap.IsValid(OtherElementIdx); OtherElementIdx = HashMap.Next(OtherElementIdx))
				{
					if (HashMap.KeyMatches(OtherElementIdx, ThisKey))
					{
						AddResult(TVec2<PayloadType>(ThisPayload, HashMap.GetValue(OtherElementIdx).Payload));
					}
				}

				// Iterate over cells within CellRadius away. Only look "forward". Particles behind us will find us by looking forward.
				// I dunno... there's probably a better way to do this.....
				for (int32 ZIndex = 1; ZIndex <= CellRadius; ++ZIndex)
				{
					AddResultsInCell(ThisPayload, FHashIndex(VectorIntAdd(ThisKey.VectorIndex, MakeVectorRegisterInt(0, 0, ZIndex, 0))));
				}

				for (int32 YIndex = 1; YIndex <= CellRadius; ++YIndex)
				{
					for (int32 ZIndex = -CellRadius; ZIndex <= CellRadius; ++ZIndex)
					{
						AddResultsInCell(ThisPayload, FHashIndex(VectorIntAdd(ThisKey.VectorIndex, MakeVectorRegisterInt(0, YIndex, ZIndex, 0))));
					}
				}

				for (int32 XIndex = 1; XIndex <= CellRadius; ++XIndex)
				{
					for (int32 YIndex = -CellRadius; YIndex <= CellRadius; ++YIndex)
					{
						for (int32 ZIndex = -CellRadius; ZIndex <= CellRadius; ++ZIndex)
						{
							AddResultsInCell(ThisPayload, FHashIndex(VectorIntAdd(ThisKey.VectorIndex, MakeVectorRegisterInt(XIndex, YIndex, ZIndex, 0))));
						}
					}
				}

			}
			
		} ConcurrentResults{ CellRadius, MaxNumExpectedConnections, NarrowTest, HashMap, ParticleIndexToElement };

		PhysicsParallelFor(ParticleIndexToElement.Num(), ConcurrentResults, bForceSingleThreaded);

		// Shrink Result array to actual number of found proximities
		const int32 ResultNum = ConcurrentResults.ResultIndex.load();
		ConcurrentResults.Result.SetNum(ResultNum, EAllowShrinking::Yes);
		return ConcurrentResults.Result;
	}
private:
	const TVectorRegisterType<T> CellSize;

	// Particle Index is dense array index passed in at build time. 
	TArray<int32> ParticleIndexToElement;
};
}

