// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Chaos/Vector.h"
#include "Chaos/Box.h"
#include "GeometryParticlesfwd.h"
#include "ChaosCheck.h"
#include "ChaosDebugDrawDeclares.h"
#include "Containers/HashTable.h"
#include "Concepts/DecaysTo.h"

namespace Chaos
{

struct FQueryFastData
{
	FQueryFastData(const FVec3& InDir, const FReal InLength)
		: Dir(InDir)
		, InvDir( (FMath::Abs(InDir[0]) < UE_SMALL_NUMBER) ? 0 : 1 / Dir[0], (FMath::Abs(InDir[1]) < UE_SMALL_NUMBER) ? 0 : 1 / Dir[1], (FMath::Abs(InDir[2]) < UE_SMALL_NUMBER) ? 0 : 1 / Dir[2])
		, bParallel{ FMath::Abs(InDir[0]) < UE_SMALL_NUMBER, FMath::Abs(InDir[1]) < UE_SMALL_NUMBER, FMath::Abs(InDir[2]) < UE_SMALL_NUMBER }
	{
		CHAOS_ENSURE(InLength != 0.0f);
		SetLength(InLength);
	}

	const FVec3& Dir;
	const FVec3 InvDir;

	FReal CurrentLength;
	FReal InvCurrentLength;

	const bool bParallel[3];

#ifdef _MSC_VER
	#pragma warning (push)
	#pragma warning(disable:4723)
#endif
	//compiler complaining about divide by 0, but we are guarding against it.
	//Seems like it's trying to evaluate div independent of the guard?

	void SetLength(const FReal InLength)
	{
		CurrentLength = InLength;

		if(InLength != 0.0f)
		{
			InvCurrentLength = 1 / InLength;
		}
	}

#ifdef _MSC_VER
	#pragma warning(pop)
#endif


protected:
	FQueryFastData(const FVec3& InDummyDir)
		: Dir(InDummyDir)
		, InvDir()
		, bParallel{}
	{}
};

//dummy struct for templatized paths
struct FQueryFastDataVoid : public FQueryFastData
{
	FQueryFastDataVoid() : FQueryFastData(DummyDir) {}
	
	FVec3 DummyDir;
};

template <typename T, int d>
class TAABB;

template <typename T, int d>
class TGeometryParticle;

template <typename T, int d>
class TSpatialRay
{
public:
	TSpatialRay()
		: Start((T)0)
		, End((T)0)
	{}

	TSpatialRay(const TVector<T, d>& InStart, const TVector<T, d> InEnd)
		: Start(InStart)
		, End(InEnd)
	{}

	TVector<T, d> Start;
	TVector<T, d> End;
};

/** A struct which is passed to spatial acceleration visitors whenever there are potential hits.
	In production, this class will only contain a payload.
*/
template <typename TPayloadType>
struct TSpatialVisitorData
{
	TPayloadType Payload;
	TSpatialVisitorData(const TPayloadType& InPayload, const bool bInHasBounds = false, const FAABB3& InBounds = FAABB3::ZeroAABB())
		: Payload(InPayload)
#if CHAOS_DEBUG_DRAW
		, bHasBounds(bInHasBounds)
		, Bounds(InBounds)
	{ }
	bool bHasBounds;
	FAABB3 Bounds;
#else
	{ }
#endif
};

/** Visitor base class used to iterate through spatial acceleration structures.
	This class is responsible for gathering any information it wants (for example narrow phase query results).
	This class determines whether the acceleration structure should continue to iterate through potential instances
*/
template <typename TPayloadType, typename T = FReal>
class ISpatialVisitor
{
public:

	virtual ~ISpatialVisitor() = default;

	/** Called whenever an instance in the acceleration structure may overlap
		@Instance - the instance we are potentially overlapping
		Returns true to continue iterating through the acceleration structure
	*/
	virtual bool Overlap(const TSpatialVisitorData<TPayloadType>& Instance) = 0;

	/** Called whenever an instance in the acceleration structure may intersect with a raycast
		@Instance - the instance we are potentially intersecting with a raycast
		@CurData - the current query data. Call SetLength to update the length all future intersection tests will use. A blocking intersection should update this
		Returns true to continue iterating through the acceleration structure
	*/
	virtual bool Raycast(const TSpatialVisitorData<TPayloadType>& Instance, FQueryFastData& CurData) = 0;

	/** Called whenever an instance in the acceleration structure may intersect with a sweep
		@Instance - the instance we are potentially intersecting with a sweep
		@CurLength - the length all future intersection tests will use. A blocking intersection should update this
		Returns true to continue iterating through the acceleration structure
	*/
	virtual bool Sweep(const TSpatialVisitorData<TPayloadType>& Instance, FQueryFastData& CurData) = 0;

	UE_DEPRECATED(5.8, "Implement PrePreFilter instead")
	virtual const void* GetQueryData() const final { return nullptr; }

	UE_DEPRECATED(5.8, "Implement PrePreFilter instead")
	virtual const void* GetSimData() const final { return nullptr; }

	UE_DEPRECATED(5.8, "Implement PrePreFilter instead")
	virtual bool ShouldIgnore(const TSpatialVisitorData<TPayloadType>& Instance) const final { return false; }

	virtual bool PrePreFilter(const TPayloadType& Payload) const { return false; }

	/** Return a pointer to the payload on which we are querying the acceleration structure */
	virtual const void* GetQueryPayload() const { return nullptr; }

	virtual bool HasBlockingHit() const { return false; }
};

/**
 * Can be implemented by external, non-chaos systems to collect / render
 * debug information from spacial structures. When passed to the debug
 * methods on ISpatialAcceleration the methods will be called out by
 * the spacial structure if implemented for the external system to handle
 * the actual drawing.
 */
template <typename T>
class ISpacialDebugDrawInterface
{
public:
	
	virtual ~ISpacialDebugDrawInterface() = default;

	virtual void Box(const TAABB<T, 3>& InBox, const TVector<T, 3>& InLinearColor, float InThickness) = 0;
	virtual void Line(const TVector<T, 3>& InBegin, const TVector<T, 3>& InEnd, const TVector<T, 3>& InLinearColor, float InThickness)  = 0;

};

using ISpatialDebugDrawInterface = ISpacialDebugDrawInterface<FReal>;

using SpatialAccelerationType = uint8;	//see ESpatialAcceleration. Projects can add their own custom types by using enum values higher than ESpatialAcceleration::Unknown
enum class ESpatialAcceleration : SpatialAccelerationType
{
	BoundingVolume,
	AABBTree,
	AABBTreeBV,
	Collection,
	Unknown,
	//For custom types continue the enum after ESpatialAcceleration::Unknown
};

inline bool SpatialAccelerationEqual(ESpatialAcceleration A, SpatialAccelerationType B) { return (SpatialAccelerationType)A == B; }
inline bool operator==(ESpatialAcceleration A, SpatialAccelerationType B) { return SpatialAccelerationEqual(A,B); }
inline bool operator==(SpatialAccelerationType A, ESpatialAcceleration B) { return SpatialAccelerationEqual(B,A); }
inline bool operator!=(ESpatialAcceleration A, SpatialAccelerationType B) { return !SpatialAccelerationEqual(A,B); }
inline bool operator!=(SpatialAccelerationType A, ESpatialAcceleration B) { return !SpatialAccelerationEqual(B,A); }

template <typename TPayload>
typename TEnableIf<!TIsPointer<TPayload>::Value, FUniqueIdx>::Type GetUniqueIdx(const TPayload& Payload)
{
	const FUniqueIdx Idx = Payload.UniqueIdx();
	ensure(Idx.IsValid());
	return Idx;
}

template <typename TPayload>
typename TEnableIf<TIsPointer<TPayload>::Value,FUniqueIdx>::Type GetUniqueIdx(const TPayload& Payload)
{
	const FUniqueIdx Idx = Payload->UniqueIdx();
	ensure(Idx.IsValid());
	return Idx;
}

FORCEINLINE FUniqueIdx GetUniqueIdx(const int32 Payload)
{
	ensure(Payload >=0);	//-1 idx implies it was never set
	return FUniqueIdx(Payload);
}

FORCEINLINE FUniqueIdx GetUniqueIdx(const FUniqueIdx Payload)
{
	ensure(Payload.IsValid());
	return Payload;
}


template <typename TPayloadType, typename T>
struct TPayloadBoundsElement
{
	TPayloadType Payload;
	TAABB<T, 3> Bounds;

	void Serialize(FChaosArchive& Ar)
	{
		Ar << Payload;
		TBox<T,3>::SerializeAsAABB(Ar, Bounds);
	}

	template <typename TPayloadType2>
	TPayloadType2 GetPayload(int32 Idx) const { return Payload; }

	bool HasBoundingBox() const { return true; }

	const TAABB<T, 3>& BoundingBox() const { return Bounds; }

	FUniqueIdx UniqueIdx() const
	{
		return ::Chaos::GetUniqueIdx(Payload);
	}
};

template <typename TPayloadType, typename T>
FChaosArchive& operator<<(FChaosArchive& Ar, TPayloadBoundsElement<TPayloadType, T>& PayloadElement)
{
	PayloadElement.Serialize(Ar);
	return Ar;
}

template<typename TPayloadType, typename T, int d>
class ISpatialAcceleration;

template<ESpatialAcceleration SpatialType, typename TPayloadType, typename T, int d>
struct TSpatialAccelerationSerializationFactory
{
	static CHAOS_API ISpatialAcceleration<TPayloadType, T, d>* Create();
};

template <typename TPayloadType, typename T, int d>
class ISpatialAcceleration
{
public:
	UE_NONCOPYABLE(ISpatialAcceleration)

	using TPayload = TPayloadType;

	ISpatialAcceleration(SpatialAccelerationType InType = static_cast<SpatialAccelerationType>(ESpatialAcceleration::Unknown))
		: SyncTimestamp(0), AsyncTimeSlicingComplete(true), Type(InType)
	{}

	ISpatialAcceleration(ESpatialAcceleration InType)
		: ISpatialAcceleration(static_cast<SpatialAccelerationType>(InType))
	{}

	virtual ~ISpatialAcceleration() = default;

	virtual bool IsAsyncTimeSlicingComplete() { return AsyncTimeSlicingComplete; }
	virtual void ProgressAsyncTimeSlicing(bool ForceBuildCompletion = false) {}
	virtual bool ShouldRebuild() { return true; }  // Used to find out if something changed since last reset for optimizations
	virtual bool IsTreeDynamic() const { return false; }  // Dynamic trees rebuild on the fly without adding dirty elements
	virtual void ClearShouldRebuild() {}
	virtual void PrepareCopyTimeSliced(const  ISpatialAcceleration<TPayloadType, T, 3>& InFrom) { check(false); }
	virtual void ProgressCopyTimeSliced(const  ISpatialAcceleration<TPayloadType, T, 3>& InFrom, int MaximumBytesToCopy) { check(false); }

	/** Cache for each leaves all the overlapping leaves */
	virtual void CacheOverlappingLeaves() {}

	// IMPORTANT : (LWC) this API should be typed on Freal not T, as we want the query API to be using the highest precision while maintaining arbitrary internal precision for the acceleration structure ( based on T )
	virtual TArray<TPayloadType> FindAllIntersections(const FAABB3& Box) const { check(false); return TArray<TPayloadType>(); }
	virtual void Raycast(const FVec3& Start, const FVec3& Dir, const FReal Length, ISpatialVisitor<TPayloadType, FReal>& Visitor) const { check(false); }
	virtual void Sweep(const FVec3& Start, const FVec3& Dir, const FReal Length, const FVec3 QueryHalfExtents, ISpatialVisitor<TPayloadType, FReal>& Visitor) const { check(false);}
	virtual void Overlap(const FAABB3& QueryBounds, ISpatialVisitor<TPayloadType, FReal>& Visitor) const { check(false); }

	virtual void Reset()
	{
		check(false);
	}

	// Returns true if element was removed successfully
	virtual bool RemoveElement(const TPayloadType& Payload)
	{
		check(false);	//not implemented
		return true;
	}

	virtual bool NeedUpdateElement(const TPayloadType& Payload, const TAABB<T, 3>& NewBounds)
	{
		return true;
	}

	virtual bool UpdateElement(const TPayloadType& Payload, const TAABB<T, d>& NewBounds, bool bHasBounds)
	{
		check(false);
		return true;
	}

	virtual bool RemoveElementFrom(const TPayloadType& Payload, FSpatialAccelerationIdx Idx)
	{
		return RemoveElement(Payload);
	}

	virtual bool NeedUpdateElementIn(const TPayloadType& Payload, const TAABB<T, d>& NewBounds, FSpatialAccelerationIdx Idx)
	{
		return NeedUpdateElement(Payload, NewBounds);
	}

	virtual bool UpdateElementIn(const TPayloadType& Payload, const TAABB<T, d>& NewBounds, bool bHasBounds, FSpatialAccelerationIdx Idx)
	{
		return UpdateElement(Payload, NewBounds, bHasBounds);
	}

	virtual TUniquePtr<ISpatialAcceleration<TPayloadType, T, d>> Copy() const
	{
		check(false);	//not implemented
		return nullptr;
	}

	virtual void DeepAssign(const ISpatialAcceleration<TPayloadType, T, d>& Other)
	{
		Type = Other.Type;
		SyncTimestamp = Other.SyncTimestamp;
		AsyncTimeSlicingComplete = Other.AsyncTimeSlicingComplete;
	}

#if !UE_BUILD_SHIPPING
	virtual void DebugDraw(ISpacialDebugDrawInterface<T>* InInterface) const {}
	virtual void DebugDrawLeaf(ISpacialDebugDrawInterface<T>& InInterface, const FLinearColor& InLinearColor, float InThickness) const {}
	virtual void DumpStats() const {}
	virtual void DumpStatsTo(class FOutputDevice& Ar) const {}
#endif

	static ISpatialAcceleration<TPayloadType, T, d>* SerializationFactory(FChaosArchive& Ar, ISpatialAcceleration<TPayloadType, T, d>* Accel)
	{
		if (Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) < FExternalPhysicsCustomObjectVersion::SerializeEvolutionGenericAcceleration)
		{
			return TSpatialAccelerationSerializationFactory<ESpatialAcceleration::BoundingVolume, TPayloadType, T, d>::Create();
		}

		int8 AccelType = Ar.IsLoading() ? 0 : (int8)Accel->Type;
		Ar << AccelType;
		switch ((ESpatialAcceleration)AccelType)
		{
		case ESpatialAcceleration::BoundingVolume: return Ar.IsLoading() ? TSpatialAccelerationSerializationFactory<ESpatialAcceleration::BoundingVolume, TPayloadType, T, d>::Create() : nullptr;
		case ESpatialAcceleration::AABBTree: return Ar.IsLoading() ? TSpatialAccelerationSerializationFactory<ESpatialAcceleration::AABBTree, TPayloadType, T, d>::Create() : nullptr;
		case ESpatialAcceleration::AABBTreeBV: return Ar.IsLoading() ? TSpatialAccelerationSerializationFactory<ESpatialAcceleration::AABBTreeBV, TPayloadType, T, d>::Create() : nullptr;
		case ESpatialAcceleration::Collection: check(false);	//Collections must be serialized directly since they are variadic
		default: check(false); return nullptr;
		}
	}

	virtual void Serialize(FChaosArchive& Ar)
	{
		check(false);
	}

	SpatialAccelerationType GetType() const { return Type; }

	template <typename TConcrete>
	TConcrete* As()
	{
		return Type == TConcrete::StaticType ? static_cast<TConcrete*>(this) : nullptr;
	}

	template <typename TConcrete>
	const TConcrete* As() const
	{
		return Type == TConcrete::StaticType ? static_cast<const TConcrete*>(this) : nullptr;
	}

	template <typename TConcrete>
	TConcrete& AsChecked()
	{
		check(Type == TConcrete::StaticType); 
		return static_cast<TConcrete&>(*this);
	}

	template <typename TConcrete>
	const TConcrete& AsChecked() const
	{
		check(Type == TConcrete::StaticType);
		return static_cast<const TConcrete&>(*this);
	}

	/** This is the time the acceleration structure is synced up with. */
	int32 GetSyncTimestamp()
	{
		return SyncTimestamp;
	}

	/** Call this whenever updating the acceleration structure for a new sync point */
	void SetSyncTimestamp(int32 InTimestamp)
	{
		SyncTimestamp = InTimestamp;
	}

protected:
	virtual void SetAsyncTimeSlicingComplete(bool InState) { AsyncTimeSlicingComplete = InState; }

private:
	int32 SyncTimestamp;	//The set of inputs the acceleration structure is in sync with. GT moves forward in time and enqueues inputs
	bool AsyncTimeSlicingComplete;
	SpatialAccelerationType Type;
};

template <typename TBase, typename TDerived>
static TUniquePtr<TDerived> AsUniqueSpatialAcceleration(TUniquePtr<TBase>&& Base)
{
	if (TDerived* Derived = Base->template As<TDerived>())
	{
		(void)Base.Release();
		return TUniquePtr<TDerived>(Derived);
	}
	return nullptr;
}

template <typename TDerived, typename TBase>
static TUniquePtr<TDerived> AsUniqueSpatialAccelerationChecked(TUniquePtr<TBase>&& Base)
{
	TDerived& Derived = Base->template AsChecked<TDerived>();
	(void)Base.Release();
	return TUniquePtr<TDerived>(&Derived);
}

/** Helper class used to bridge virtual to template implementation of acceleration structures */
template <typename TPayloadType, typename T>
class TSpatialVisitor
{
public:
	TSpatialVisitor(ISpatialVisitor<TPayloadType, T>& InVisitor)
		: Visitor(InVisitor) {}
	FORCEINLINE bool VisitOverlap(const TSpatialVisitorData<TPayloadType>& Instance)
	{
		return Visitor.Overlap(Instance);
	}

	FORCEINLINE bool VisitRaycast(const TSpatialVisitorData<TPayloadType>& Instance, FQueryFastData& CurData)
	{
		return Visitor.Raycast(Instance, CurData);
	}

	FORCEINLINE bool VisitSweep(const TSpatialVisitorData<TPayloadType>& Instance, FQueryFastData& CurData)
	{
		return Visitor.Sweep(Instance, CurData);
	}

	UE_DEPRECATED(5.8, "If this is errors, your visitor is missing an implementation of PrePreFilter")
	FORCEINLINE const void* GetQueryData() const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return Visitor.GetQueryData();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	UE_DEPRECATED(5.8, "If this is errors, your visitor is missing an implementation of PrePreFilter")
	FORCEINLINE const void* GetSimData() const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return Visitor.GetSimData();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	UE_DEPRECATED(5.8, "If this is errors, your visitor is missing an implementation of PrePreFilter")
	FORCEINLINE bool ShouldIgnore(const TSpatialVisitorData<TPayloadType>& Instance) const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return Visitor.ShouldIgnore(Instance);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	FORCEINLINE bool PrePreFilter(const TPayloadType& Payload) const
	{
		return Visitor.PrePreFilter(Payload);
	}

	/** Return a pointer to the payload on which we are querying the acceleration structure */
	FORCEINLINE const void* GetQueryPayload() const
	{
		return Visitor.GetQueryPayload();
	}

	FORCEINLINE bool HasBlockingHit() const
	{
		return Visitor.HasBlockingHit();
	}

private:
	ISpatialVisitor<TPayloadType, T>& Visitor;
};

#ifndef CHAOS_SERIALIZE_OUT
#define CHAOS_SERIALIZE_OUT WITH_EDITOR
#endif

//Provides a TMap like API but backed by a dense array. The keys provided must implement GetUniqueIdx
template <typename TKey, typename TValue>
class TArrayAsMap
{
	struct FEntry
	{
		TValue Value;
		bool bSet;

		FEntry()
			: bSet(false)
		{

		}
	};

public:
	// @todo(chaos): rename with "F"
	using ElementType = TValue;

	static constexpr uint32 GetTypeSize()
	{
		return sizeof(FEntry);
	}

	SIZE_T GetAllocatedSize() const
	{
		SIZE_T AllocatedSize = Entries.GetAllocatedSize();

#if CHAOS_SERIALIZE_OUT
		AllocatedSize += KeysToSerializeOut.GetAllocatedSize();
#endif

		return AllocatedSize;
	}

	int32 Capacity() const
	{
		return Entries.Max();
	}

	// @todo(chaos): rename with "F"
	struct Element
	{
#if CHAOS_SERIALIZE_OUT
		TKey KeyToSerializeOut;
#endif
		TValue Entry;
	};

	int32 Num() const
	{
		return Entries.Num();
	}

	void Reserve(int32 Size)
	{
		Entries.Reserve(Size);
#if CHAOS_SERIALIZE_OUT
		KeysToSerializeOut.Reserve(Size);
#endif
	}

	TValue* Find(const TKey& Key)
	{
		const int32 Idx = GetUniqueIdx(Key).Idx;
		if(Entries.IsValidIndex(Idx) && Entries[Idx].bSet)
		{
			return &Entries[Idx].Value;
		}
		return nullptr;
	}

	const TValue* Find(const TKey& Key) const
	{
		const int32 Idx = GetUniqueIdx(Key).Idx;
		if(Idx < Entries.Num() && Entries[Idx].bSet)
		{
			return &Entries[Idx].Value;
		}
		return nullptr;
	}

	TValue& FindChecked(const TKey& Key)
	{
		return Entries[GetUniqueIdx(Key).Idx].Value;
	}

	const TValue& FindChecked(const TKey& Key) const
	{
		return Entries[GetUniqueIdx(Key).Idx].Value;
	}

	TValue& FindOrAdd(const TKey& Key)
	{
		if(TValue* Elem = Find(Key))
		{
			return *Elem;
		}

		return Add(Key);
	}

	void Empty()
	{
		Entries.Empty();
	}

	TValue& Add(const TKey& Key)
	{
		const int32 Idx = GetUniqueIdx(Key).Idx;
		if(Idx >= Entries.Num())
		{
			const int32 NumToAdd = Idx + 1 - Entries.Num();
			Entries.AddDefaulted(NumToAdd);
#if CHAOS_SERIALIZE_OUT
			KeysToSerializeOut.AddDefaulted(NumToAdd);
#endif
		}

		ensure(Entries[Idx].bSet == false);	//element already added
		Entries[Idx].bSet = true;

#if CHAOS_SERIALIZE_OUT
		KeysToSerializeOut[Idx] = Key;
#endif

		return Entries[Idx].Value;
	}

	void Add(const TKey& Key, const TValue& Value)
	{
		Add(Key) = Value;
	}

	void RemoveChecked(const TKey& Key)
	{
		const int32 Idx = GetUniqueIdx(Key).Idx;
		Entries[Idx] = FEntry();	//Mark as free, also resets default values for next use of value
#if CHAOS_SERIALIZE_OUT
		KeysToSerializeOut[Idx] = TKey();
#endif
	}

	void Remove(const TKey& Key)
	{
		const int32 Idx = GetUniqueIdx(Key).Idx;
		if(Idx >= 0 && Idx < Entries.Num())
		{
			Entries[Idx] = FEntry();	//Mark as free, also resets default values for next use of value
#if CHAOS_SERIALIZE_OUT
			KeysToSerializeOut[Idx] = TKey();
#endif
		}
	}

	void Reset()
	{
		Entries.Reset();
#if CHAOS_SERIALIZE_OUT 
		KeysToSerializeOut.Reset();
#endif
	}

	void Serialize(FChaosArchive& Ar)
	{
		bool bCanSerialize = Ar.IsLoading();
#if CHAOS_SERIALIZE_OUT 
		bCanSerialize = true;
#endif

		if(bCanSerialize)
		{
			TArray<TKey> DirectKeys;
			Ar << DirectKeys;

			for(auto& Key : DirectKeys)
			{
				TValue& Value = Add(Key);
				Ar << Value;
			}
		}
		else
		{
			CHAOS_ENSURE(false);	//can't serialize out, if you are trying to serialize for perf/debug set CHAOS_SERIALIZE_OUT to 1 
		}
	}

	void AddFrom(const TArrayAsMap<TKey, TValue>& Source, int32 SourceIndex)
	{
		Entries.Add(Source.Entries[SourceIndex]);
#if CHAOS_SERIALIZE_OUT
		KeysToSerializeOut.Add(Source.KeysToSerializeOut[SourceIndex]);
#endif
	}

private:

	TArray<FEntry> Entries;

#if CHAOS_SERIALIZE_OUT
	//The indices are generated at runtime, so there's no way to serialize them directly
	//Because of that we serialize the actual key which we can find, and then at runtime we use its transient index
	TArray<TKey> KeysToSerializeOut;
#endif
};

template <typename TKey, typename TValue>
struct SQMapKeyWithValue
{
	SQMapKeyWithValue() = default;
	SQMapKeyWithValue(const TKey& InKey, const TValue& InValue)
		: Key(InKey)
		, Value(InValue)
	{}

	TKey Key;
	TValue Value;
};

template <typename TKey, typename TValue>
FChaosArchive& operator<< (FChaosArchive& Ar, SQMapKeyWithValue<TKey, TValue>& Pair)
{
	Ar << Pair.Key;
	Ar << Pair.Value;

	return Ar;
}

class FSQHashTable : public FHashTable
{
public:
	SIZE_T GetAllocatedSize() const
	{
		return IndexSize == 0 ? 0 : (IndexSize + HashSize) * sizeof(uint32);
	}

	uint32 GetIndexSize() const
	{
		return IndexSize;
	}
};

/**
 * Map structure using FHashTable as a base store for payload data
 * FHashTable is a fast, limited API map that requires some management.
 * In this case as we already have unique values we can just mix the bits
 * to get a reasonable distribution (the calls to MurmurFinalize32).
 */
template <typename TKey, typename TValue>
class TSQMap
{
public:

	using ElementType = TValue;
	using PairType = SQMapKeyWithValue<TKey, TValue>;

	static constexpr uint32 GetTypeSize()
	{
		return sizeof(PairType);
	}

	SIZE_T GetAllocatedSize() const
	{
		return Elements.GetAllocatedSize() + HashTable.GetAllocatedSize();
	}

	int32 Num() const
	{
		return Elements.Num();
	}

	int32 Capacity() const
	{
		return HashTable.GetIndexSize();
	}

	void Reserve(int32 NumToReserve)
	{
		Elements.Reserve(NumToReserve);

		// FHashTable requires powers of two but doesn't assert it.
		const uint32 RequiredSize = FMath::RoundUpToPowerOfTwo(NumToReserve + 1);

		// #TODO should we allow SQ shrinking?
		if(RequiredSize > HashTable.GetIndexSize())
		{
			HashTable.Resize(RequiredSize);
		}
	}

	void ResizeHashBuckets(uint32 NewSize)
	{
		const uint32 NumElements = Elements.Num();
		HashTable.Clear(NewSize, FMath::RoundUpToPowerOfTwo(NumElements + 1));

		// Rehash the elements, we don't use the other Add function here
		// as that will search for the element and we know it's not in here
		// as we're building a new hash table
		for(uint32 ElemIdx = 0; ElemIdx < NumElements; ++ElemIdx)
		{
			const uint32 ElemHash = MurmurFinalize32(GetUniqueIdx(Elements[ElemIdx].Key).Idx);
			HashTable.Add(ElemHash, ElemIdx);
		}
	}

	TValue* Find(const TKey& Key)
	{
		int32 Index = FindIndex(Key);

		if(Index != INDEX_NONE)
		{
			return &Elements[Index].Value;
		}

		return nullptr;
	}

	const TValue* Find(const TKey& Key) const
	{
		int32 Index = FindIndex(Key);

		if(Index != INDEX_NONE)
		{
			return &Elements[Index].Value;
		}

		return nullptr;
	}

	TValue& FindChecked(const TKey& Key)
	{
		int32 Index = FindIndex(Key);
		check(Index != INDEX_NONE);
		return Elements[Index].Value;
	}

	const TValue& FindChecked(const TKey& Key) const
	{
		int32 Index = FindIndex(Key);
		check(Index != INDEX_NONE);
		return Elements[Index].Value;
	}

	TValue& FindOrAdd(const TKey& Key)
	{
		const int32 KeyIndex = GetUniqueIdx(Key).Idx;
		const uint32 Hash = MurmurFinalize32(KeyIndex);
		int32 Index = FindIndex(Hash, KeyIndex);
		
		if(Index == INDEX_NONE)
		{
			return Add(Hash, Key);
		}

		return Elements[Index].Value;
	}

	void Empty()
	{
		Elements.Empty();
		HashTable.Free();
	}

	FORCEINLINE TValue& Add(const TKey& Key)
	{
		return Add(MurmurFinalize32(GetUniqueIdx(Key).Idx), Key);
	}

	FORCEINLINE TValue& Add(uint32 InHash, const TKey& Key)
	{
		int32 ExistingIndex = FindIndex(InHash, GetUniqueIdx(Key).Idx);

		if(ExistingIndex == INDEX_NONE)
		{
			Elements.Emplace(Key, TValue{});
			ExistingIndex = Elements.Num() - 1;
			HashTable.Add(InHash, ExistingIndex);
		}

		return Elements[ExistingIndex].Value;
	}

	void Add(const TKey& Key, const TValue& Value)
	{
		Add(Key) = Value;
	}

	void RemoveChecked(const TKey& Key)
	{
		const int32 NumBefore = Elements.Num();
		Remove(Key);
		check(NumBefore > Elements.Num());
	}

	void Remove(const TKey& Key)
	{
		const int32 KeyIndex = GetUniqueIdx(Key).Idx;
		const uint32 Hash = MurmurFinalize32(KeyIndex);
		const int32 Index = FindIndex(Hash, KeyIndex);

		if(Index != INDEX_NONE)
		{
			HashTable.Remove(Hash, Index);

			// Before removing from the array, if we have more than 1 element we
			// swap the back element into the empty slot and rehash it. Maybe
			// we should use a freelist? (profile this)
			const int32 NumElems = Elements.Num();
			const int32 BackIndex = NumElems - 1;
			if(NumElems > 1)
			{
				if(Index == BackIndex)
				{
					// If we're already the back element, there's nothing to re-add
					Elements.RemoveAtSwap(Index, EAllowShrinking::No);
				}
				else
				{
					PairType& BackPair = Elements[NumElems - 1];
					const uint32 BackHash = MurmurFinalize32(GetUniqueIdx(BackPair.Key).Idx);
					HashTable.Remove(BackHash, NumElems - 1);
					Elements.RemoveAtSwap(Index, EAllowShrinking::No);
					HashTable.Add(BackHash, Index);
				}
			}
			else
			{
				Elements.Reset();
			}
		}
	}

	void Reset()
	{
		Elements.Reset();
		HashTable.Clear();
	}

	FORCEINLINE int32 FindIndex(const TKey& Key) const
	{
		const int32 KeyIndex = GetUniqueIdx(Key).Idx;
		return FindIndex(MurmurFinalize32(KeyIndex), KeyIndex);
	}

	FORCEINLINE int32 FindIndex(uint32 InHash, int32 UniqueIndex) const
	{
		for(uint32 i = HashTable.First(InHash); HashTable.IsValid(i); i = HashTable.Next(i))
		{
			if(GetUniqueIdx(Elements[i].Key).Idx == UniqueIndex)
			{
				return i;
			}
		}

		return INDEX_NONE;
	}

	void Serialize(FChaosArchive& Ar)
	{
		Ar << Elements;

		if(Ar.IsLoading())
		{
			const uint32 NumElements = Elements.Num();
			HashTable.Clear(1024, FMath::RoundUpToPowerOfTwo(NumElements));
			for(uint32 i = 0; i < NumElements; ++i)
			{
				HashTable.Add(MurmurFinalize32(GetUniqueIdx(Elements[i].Key).Idx), i);
			}
		}
	}

	void AddFrom(const TSQMap<TKey, TValue>& Source, int32 SourceIndex)
	{
		const PairType& SourcePair = Source.Elements[SourceIndex];
		Add(SourcePair.Key, SourcePair.Value);
	}

	TArray<PairType> Elements;
	FSQHashTable HashTable;
};

template <typename TKey, typename TValue>
FChaosArchive& operator<< (FChaosArchive& Ar, TArrayAsMap<TKey, TValue>& Map)
{
	Map.Serialize(Ar);
	return Ar;
}

template <typename TKey, typename TValue>
FChaosArchive& operator<< (FChaosArchive& Ar, TSQMap<TKey, TValue>& Map)
{
	Map.Serialize(Ar);
	return Ar;
}

template <typename PayloadType, typename VisitorType>
concept CVisitorHasPrePreFilter = requires(PayloadType Payload, VisitorType Visitor)
{
    { Visitor.PrePreFilter(Payload) } -> UE::CDecaysTo<bool>;
};

template <typename PayloadType, typename VisitorType>
UE_DEPRECATED(5.8, "Implement the new PrePreFilter method on your visitor")
UE_INTERNAL bool PrePreFilterHelperInternal(const PayloadType& Payload, const VisitorType& Visitor)
{
	if (Visitor.ShouldIgnore(Payload))
	{
		return true;
	}
	if (const void* QueryData = Visitor.GetQueryData())
	{
		return Payload.PrePreQueryFilter(QueryData);
	}
	if (const void* SimData = Visitor.GetSimData())
	{
		return Payload.PrePreSimFilter(SimData);
	}
	return false;
}

template <typename PayloadType, typename VisitorType>
requires CVisitorHasPrePreFilter<PayloadType, VisitorType>
UE_INTERNAL bool PrePreFilterHelperInternal(const PayloadType& Payload, const VisitorType& Visitor)
{
	return Visitor.PrePreFilter(Payload);
}

template <typename TPayload, typename TVisitor>
typename TEnableIf<!TIsPointer<TPayload>::Value, bool>::Type PrePreFilterHelper(const TPayload& Payload, const TVisitor& Visitor)
{
	PRAGMA_DISABLE_INTERNAL_WARNINGS
	return PrePreFilterHelperInternal(Payload, Visitor);
	PRAGMA_ENABLE_INTERNAL_WARNINGS
}

template <typename TPayload, typename TVisitor>
typename TEnableIf<TIsPointer<TPayload>::Value, bool>::Type PrePreFilterHelper(const TPayload& Payload, const TVisitor& Visitor)
{
	return false;
}

template <typename TVisitor>
FORCEINLINE bool PrePreFilterHelper(const int32 Payload, const TVisitor& Visitor)
{
	return false;
}

#if !UE_MERGED_MODULES
#if PLATFORM_COMPILER_CLANG
extern template class CHAOS_API ISpatialAcceleration<int32, FReal, 3>;
extern template class CHAOS_API ISpatialVisitor<int32, FReal>;
#else
extern template class ISpatialAcceleration<int32, FReal, 3>;
extern template class ISpatialVisitor<int32, FReal>;
#endif
#endif

}
