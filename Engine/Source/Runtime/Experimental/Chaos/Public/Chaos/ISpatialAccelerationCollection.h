// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Chaos/ISpatialAcceleration.h"
#include "Chaos/Box.h"
#include "Chaos/Collision/StatsData.h"
#include "GeometryParticlesfwd.h"

#include <tuple>

namespace Chaos
{
	namespace Private
	{
		class FCollisionConstraintAllocator;
	}
	class FAsyncCollisionReceiver;
	class FCollisionDetectorSettings;
	class FSpatialAccelerationBroadPhase;
	class IResimCacheBase;

template <typename TPayloadType, typename T, int d>
class ISpatialAccelerationCollection : public ISpatialAcceleration<TPayloadType, T, d>
{
public:
	UE_NONCOPYABLE(ISpatialAccelerationCollection)

	ISpatialAccelerationCollection()
	: ISpatialAcceleration<TPayloadType, T, d>(StaticType)
	, ActiveBucketsMask(0)
	, AllAsyncTasksComplete(true)
	{}
	static constexpr ESpatialAcceleration StaticType = ESpatialAcceleration::Collection;
	virtual FSpatialAccelerationIdx AddSubstructure(TUniquePtr<ISpatialAcceleration<TPayloadType, T, d>>&& Substructure, uint16 Bucket, uint16 BucketInnerIdx) = 0;
	virtual TUniquePtr<ISpatialAcceleration<TPayloadType, T, d>> RemoveSubstructure(FSpatialAccelerationIdx Idx) = 0;
	virtual ISpatialAcceleration<TPayloadType, T, d>* GetSubstructure(FSpatialAccelerationIdx Idx) = 0;
	virtual void SwapSubstructure(ISpatialAccelerationCollection<TPayloadType, T, d>& Other, FSpatialAccelerationIdx Idx) = 0;	

	/** This is kind of a hack to avoid virtuals. We simply route calls into templated functions */
	virtual void PBDComputeConstraintsLowLevel(T Dt, FSpatialAccelerationBroadPhase& BroadPhase, Private::FCollisionConstraintAllocator* Allocator, const FCollisionDetectorSettings& Settings, IResimCacheBase* ResimCache) const = 0;
	virtual TArray<FSpatialAccelerationIdx> GetAllSpatialIndices() const = 0;

	bool IsBucketActive(uint8 BucketIdx) const
	{
		return (1 << BucketIdx) & ActiveBucketsMask;
	}

	bool IsAllAsyncTasksComplete() const { return AllAsyncTasksComplete; }
	void SetAllAsyncTasksComplete(bool State) { AllAsyncTasksComplete = State; }

	void DeepAssign(const ISpatialAccelerationCollection<TPayloadType, FReal, 3>& Other)
	{
		ISpatialAcceleration<TPayloadType, FReal, 3>::DeepAssign(Other);
		check(ActiveBucketsMask == Other.ActiveBucketsMask);
		AllAsyncTasksComplete = Other.AllAsyncTasksComplete;
	}

	virtual void DeepAssign(const ISpatialAcceleration<TPayloadType, FReal, 3>& Other) override
	{
		check(false);	//not implemented
	}

#if !UE_BUILD_SHIPPING
	virtual void DebugDraw(ISpacialDebugDrawInterface<T>* InInterface) const = 0;
#endif

protected:
	uint8 ActiveBucketsMask;
	bool AllAsyncTasksComplete;
};

}
