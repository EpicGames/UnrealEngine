// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Chaos/Core.h"

namespace Chaos
{	
	template<class T, int d> class TBox;
	template<class T, int d> class TPlane;
	template<class T, int d> class TSphere;
	template<class T> class TTriangle;

	class FConvex;
	class FCapsule;
	class FHeightField;
	class FImplicitObject;
	class FLevelSet;
	class FMLLevelSet;
	class FTriangleMeshImplicitObject;
	class FImplicitObjectUnionClustered;
	class FImplicitObjectUnion;
	class FSkinnedTriangleMesh;

	using FImplicitBox3 = TBox<FReal, 3>;
	using FImplicitCapsule3 = FCapsule;
	using FImplicitConvex3 = FConvex;
	using FImplicitHeightField3 = FHeightField;
	using FImplicitObject3 = FImplicitObject;
	using FImplicitPlane3 = TPlane<FReal, 3>;
	using FImplicitSphere3 = TSphere<FReal, 3>;

	using FTriangle = TTriangle<FReal>;
	
	using FImplicitObjectPtr = TRefCountPtr<FImplicitObject>;
	using FConstImplicitObjectPtr = TRefCountPtr<const FImplicitObject>;
	using FConvexPtr = TRefCountPtr<FConvex>;
	using FTriangleMeshImplicitObjectPtr = TRefCountPtr<FTriangleMeshImplicitObject>;
	using FLevelSetPtr = TRefCountPtr<FLevelSet>;
	using FMLLevelSetPtr = TRefCountPtr<FMLLevelSet>; 
	using FHeightFieldPtr = TRefCountPtr<FHeightField>;
	using FCapsulePtr = TRefCountPtr<FCapsule>;
	using FPlanePtr = TRefCountPtr<FImplicitPlane3>;
	using FBoxPtr = TRefCountPtr<FImplicitBox3>;
	using FSpherePtr = TRefCountPtr<FImplicitSphere3>;
	using FImplicitObjectUnionClusteredPtr = TRefCountPtr<FImplicitObjectUnionClustered>;
	using FImplicitObjectUnionPtr = TRefCountPtr<FImplicitObjectUnion>;
	using FSkinnedTriangleMeshPtr = TRefCountPtr<FSkinnedTriangleMesh>;
	
	using FImplicitObjectRef = FImplicitObject*;
	using FConstImplicitObjectRef = const FImplicitObject*;
	using FConvexRef = FConvex*;
	using FTriangleMeshImplicitObjectRef = FTriangleMeshImplicitObject*;
	using FLevelSetRef = FLevelSet*;
	using FMLLevelSetRef = FMLLevelSet*;
	using FHeightFieldRef = FHeightField*;
	using FCapsuleRef = FCapsule*;
	using FPlaneRef = FImplicitPlane3*;
	using FBoxRef = FImplicitBox3*;
	using FSphereRef = FImplicitSphere3*;
	using FImplicitObjectUnionClusteredRef = FImplicitObjectUnionClustered*;
	using FImplicitObjectUnionRef = FImplicitObjectUnion*;
	using FSkinnedTriangleMeshRef = FSkinnedTriangleMesh*;
	
	using FImplicitObjectsArray = TArray<FImplicitObjectPtr>;
}