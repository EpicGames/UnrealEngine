// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include <atomic>
#include "Chaos/Pair.h"
#include "Chaos/Serializable.h"
#include "Chaos/Core.h"
#include "Chaos/ImplicitFwd.h"
#include "Chaos/ImplicitObjectType.h"
#include "Chaos/AABB.h"
#include "Chaos/RefCountedObject.h"
#include "Templates/RefCounting.h"
#include "AutoRTFM.h"

#ifndef TRACK_CHAOS_GEOMETRY
#define TRACK_CHAOS_GEOMETRY !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
#endif

namespace Chaos
{
template<class T, int d>
class TAABB;
class FCylinder;
template<class T, int d>
class TSphere;
template<class T, int d>
class TPlane;
template<class T, int d>
class TParticles;

class FBVHParticles;
class FImplicitObject;

using FAABB3 = TAABB<FReal, 3>;
using FParticles = TParticles<FReal, 3>;
using FSphere = TSphere<FReal, 3>; // warning: code assumes that FImplicitObjects with type ImplicitObjectType::Sphere are FSpheres, but all TSpheres will think they have ImplicitObjectType::Sphere.
using FSpheref = TSphere<FRealSingle, 3>;


template<class T, int d, bool bSerializable>
struct TImplicitObjectPtrStorage
{
};

template<class T, int d>
struct TImplicitObjectPtrStorage<T, d, false>
{
	using PtrType = FImplicitObjectRef;

	static PtrType Convert(const Chaos::FImplicitObjectPtr& Object)
	{
		return Object.GetReference();
	}
};

template<class T, int d>
struct TImplicitObjectPtrStorage<T, d, true>
{
	using PtrType = FImplicitObjectPtr;

	static PtrType Convert(const Chaos::FImplicitObjectPtr& Object)
	{
		return Object;
	}
};

/**
* A visitor for use in FImplicitObject hierarchy visiting functions.
* @param Implicit The geometry we are currently visiting
* @param Transform The net transform relative to the hierarchy root (the originating visit call)
* @param RootObjectIndex The index of our ancestor in the root union. Will be INDEX_NONE if no Union at the root. Used to index into a Particle's ShapeInstances or get our ancenstor.
* @param ObjectIndex A counter tracking the current implicit object index in the flattened hierarchy (pre-order, depth first)
* @param LeafObjectIndex A counter tracking the current leaf index in the flattened hierarchy (pre-order, depth first). Used to differentiate between geometry when duplicated in the hierarchy. INDEX_NONE if not visiting a leaf.
*/
using FImplicitHierarchyVisitor = TFunctionRef<void(const FImplicitObject* Implicit, const FRigidTransform3& Transform, const int32 RootObjectIndex, const int32 ObjectIndex, const int32 LeafObjectIndex)>;
using FImplicitHierarchyVisitorBool = TFunctionRef<bool(const FImplicitObject* Implicit, const FRigidTransform3& Transform, const int32 RootObjectIndex, const int32 ObjectIndex, const int32 LeafObjectIndex)>;


// Specialized for derived classes so that we can downcast to non-leaf types in the class hierarchy
// @see TImplicitTypeInfo<FImplicitObjectUnion>
template<typename T>
struct TImplicitTypeInfo
{
	// Return true if implicits of type InType can be cast to T.
	// I.e., is T a base class (or the class) of InType.
	static bool IsBaseOf(const EImplicitObjectType InType)
	{
		return (T::StaticType() == InType);
	}
};

/*
 * Base class for implicit collision geometry such as spheres, capsules, boxes, etc.
 * 
 * Some shapes are represented by a core shape with a margin. E.g. Spheres are
 * a point with a margin equal to the radius; boxes are a core AABB with a margin.
 * The margin is considered to be physically part of the shape for the pupose
 * of collision detection, separating distance, etc.
 * 
 * The margin exists to make GJK and EPA collision more robust, and is not required
 * by all derived types. E.g., We never use GJK to perform triangle-triangle collision
 * so we do not need a margin on triangles. We would need a margin on every type
 * that can be tested against a triangle though.
 */
// Some collision ispc code requires that no tail padding in FImplicitObject is reused by derived class members. 
// This is a compiler-dependent behavior, so if you are not seeing any other compile time errors about sizeof(FImplicitObject) + offsetof(...) with this disabled,
// you should be OK.
#define DISALLOW_FIMPLICIT_OBJECT_TAIL_PADDING INTEL_ISPC

class FImplicitObject : public FChaosRefCountedObject
{
public:
	using TType = FReal;
	static constexpr int D = 3;
	static CHAOS_API FImplicitObject* SerializationFactory(FChaosArchive& Ar, FImplicitObject* Obj);

	CHAOS_API FImplicitObject(int32 Flags, EImplicitObjectType InType = ImplicitObjectType::Unknown);
	FImplicitObject(const FImplicitObject&) = delete;
	FImplicitObject(FImplicitObject&&) = delete;
	CHAOS_API virtual ~FImplicitObject() override;

	// Can this object be cast to type T_DERIVED?
	template<typename TargetType>
	bool IsA() const
	{
		return TImplicitTypeInfo<TargetType>::IsBaseOf(GetType());
	}

	// Dynamic cast to type T_DERIVED. Returns null if T_DERIVED is not a valid cast for this object.
	template<typename TargetType>
	const TargetType* AsA() const
	{
		if (IsA<TargetType>())
		{
			return static_cast<const TargetType*>(this);
		}
		return nullptr;
	}

	template<typename TargetType>
	TargetType* AsA()
	{
		if (IsA<TargetType>())
		{
			return static_cast<TargetType*>(this);
		}
		return nullptr;
	}

	template<typename TargetType>
	const TargetType* AsAChecked() const
	{
		check(IsA<TargetType>());
		return static_cast<const TargetType*>(this);
	}

	template<typename TargetType>
	TargetType* AsAChecked()
	{
		check(IsA<TargetType>());
		return static_cast<TargetType*>(this);
	}

	template<class T_DERIVED>
	T_DERIVED* GetObject()
	{
		if (T_DERIVED::StaticType() == Type)
		{
			return static_cast<T_DERIVED*>(this);
		}
		return nullptr;
	}

	template<class T_DERIVED>
	const T_DERIVED* GetObject() const
	{
		if (T_DERIVED::StaticType() == Type)
		{
			return static_cast<const T_DERIVED*>(this);
		}
		return nullptr;
	}

	template<class T_DERIVED>
	const T_DERIVED& GetObjectChecked() const
	{
		check(T_DERIVED::StaticType() == Type);
		return static_cast<const T_DERIVED&>(*this);
	}

	template<class T_DERIVED>
	T_DERIVED& GetObjectChecked()
	{
		check(T_DERIVED::StaticType() == Type);
		return static_cast<T_DERIVED&>(*this);
	}

	virtual EImplicitObjectType GetNestedType() const { return GetType(); }
	CHAOS_API EImplicitObjectType GetType() const;
	static int32 GetOffsetOfType() { return offsetof(FImplicitObject, Type); }

	CHAOS_API EImplicitObjectType GetCollisionType() const;
	
	void SetCollisionType(EImplicitObjectType InCollisionType) { CollisionType = InCollisionType; }
	

	UE_DEPRECATED(5.6, "Please use GetRadiusf instead.")
	virtual FReal GetRadius() const { return 0.0; }
	virtual FRealSingle GetRadiusf() const { return 0.0f; }
	 UE_DEPRECATED(5.6, "Please use GetMarginf instead.")
	virtual FReal GetMargin() const { return Margin; }
	virtual FRealSingle GetMarginf() const { return Margin; }
	
	static int32 GetOffsetOfMargin() { return offsetof(FImplicitObject, Margin); }

	CHAOS_API virtual bool IsValidGeometry() const;

	CHAOS_API virtual Chaos::FImplicitObjectPtr CopyGeometry() const;
	CHAOS_API virtual Chaos::FImplicitObjectPtr CopyGeometryWithScale(const FVec3& Scale) const;
	virtual Chaos::FImplicitObjectPtr DeepCopyGeometry() const { return CopyGeometry(); } 
	virtual Chaos::FImplicitObjectPtr DeepCopyGeometryWithScale(const FVec3& Scale) const { return CopyGeometryWithScale(Scale); }

	UE_DEPRECATED(5.4, "Please use DeepCopyGeometry instead")
	virtual FImplicitObject* Duplicate() const { check(false); return nullptr; }
	
	UE_DEPRECATED(5.4, "Please use CopyGeometry instead")
	virtual TUniquePtr<FImplicitObject> Copy() const { check(false); return nullptr; }
	
	UE_DEPRECATED(5.4, "Please use CopyGeometryWithScale instead")
    virtual TUniquePtr<FImplicitObject> CopyWithScale(const FVec3& Scale) const { check(false); return nullptr; }
    
	UE_DEPRECATED(5.4, "Please use DeepCopyGeometry instead")
    virtual TUniquePtr<FImplicitObject> DeepCopy() const { check(false); return nullptr; }
    
	UE_DEPRECATED(5.4, "Please use DeepCopyGeometryWithScale instead")
    virtual TUniquePtr<FImplicitObject> DeepCopyWithScale(const FVec3& Scale) const { check(false); return nullptr; }

	//This is strictly used for optimization purposes
	CHAOS_API bool IsUnderlyingUnion() const;

	//This is strictly used for optimization purposes
	CHAOS_API bool IsUnderlyingMesh() const;

	// Explicitly non-virtual.  Must cast to derived types to target their implementation.
	CHAOS_API FReal SignedDistance(const FVec3& x) const;

	// Explicitly non-virtual.  Must cast to derived types to target their implementation.
	CHAOS_API FVec3 Normal(const FVec3& x) const;

	// Find the closest point on the surface, and return the separating distance and axis
	virtual FReal PhiWithNormal(const FVec3& x, FVec3& Normal) const = 0;

	// Find the closest point on the surface, and return the separating distance and axis
	virtual FReal PhiWithNormalScaled(const FVec3& Pos, const FVec3& Scale, FVec3& Normal) const
	{
		// @todo(chaos): implement for all derived types - this is not a valid solution
		const FVec3 UnscaledX = Pos / Scale;
		FVec3 UnscaledNormal;
		const FReal UnscaledPhi = PhiWithNormal(UnscaledX, UnscaledNormal);
		Normal = Scale * UnscaledNormal;
		const FReal ScaleFactor = Normal.SafeNormalize();
		const FReal ScaledPhi = UnscaledPhi * ScaleFactor;
		return ScaledPhi;
	}

	CHAOS_API virtual const FAABB3 BoundingBox() const;

	// Calculate the tight-fitting world-space bounding box
	virtual FAABB3 CalculateTransformedBounds(const FRigidTransform3& Transform) const
	{
		check(HasBoundingBox());
		return BoundingBox().TransformedAABB(Transform);
	}

	bool HasBoundingBox() const { return bHasBoundingBox; }

	bool IsConvex() const { return bIsConvex; }
	void SetConvex(const bool Convex = true) { bIsConvex = Convex; }

	void SetDoCollide(const bool Collide ) { bDoCollide = Collide; }
	bool GetDoCollide() const { return bDoCollide; }
	
#if TRACK_CHAOS_GEOMETRY
	//Turn on memory tracking. Must pass object itself as a serializable ptr so we can save it out
	CHAOS_API void Track(TSerializablePtr<FImplicitObject> This, const FString& DebugInfo);
#endif

	virtual bool IsPerformanceWarning() const { return false; }
	virtual FString PerformanceWarningAndSimplifaction() 
	{
		return FString::Printf(TEXT("ImplicitObject - No Performance String"));
	};

	CHAOS_API Pair<FVec3, bool> FindDeepestIntersection(const FImplicitObject* Other, const FBVHParticles* Particles, const FMatrix33& OtherToLocalTransform, const FReal Thickness) const;
	CHAOS_API Pair<FVec3, bool> FindDeepestIntersection(const FImplicitObject* Other, const FParticles* Particles, const FMatrix33& OtherToLocalTransform, const FReal Thickness) const;
	CHAOS_API Pair<FVec3, bool> FindClosestIntersection(const FVec3& StartPoint, const FVec3& EndPoint, const FReal Thickness) const;

	//This gives derived types a way to avoid calling PhiWithNormal todo: this api is confusing
	virtual bool Raycast(const FVec3& StartPoint, const FVec3& Dir, const FReal Length, const FReal Thickness, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex) const
	{
		OutFaceIndex = INDEX_NONE;
		const FVec3 EndPoint = StartPoint + Dir * Length;
		Pair<FVec3, bool> Result = FindClosestIntersection(StartPoint, EndPoint, Thickness);
		if (Result.Second)
		{
			OutPosition = Result.First;
			OutNormal = Normal(Result.First);
			OutTime = Length > 0 ? (OutPosition - StartPoint).Size() : 0.f;
			return true;
		}
		return false;
	}

	/** Returns the most opposing face.
		@param Position - local position to search around (for example an edge of a convex hull)
		@param UnitDir - the direction we want to oppose (for example a ray moving into the edge of a convex hull would get the face with the most negative dot(FaceNormal, UnitDir)
		@param HintFaceIndex - for certain geometry we can use this to accelerate the search.
		@return Index of the most opposing face
	*/
	virtual int32 FindMostOpposingFace(const FVec3& Position, const FVec3& UnitDir, int32 HintFaceIndex, FReal SearchDist) const
	{
		//Many objects have no concept of a face
		return INDEX_NONE;
	}

	virtual int32 FindMostOpposingFaceScaled(const FVec3& Position, const FVec3& UnitDir, int32 HintFaceIndex, FReal SearchDist, const FVec3& Scale) const
	{
		// For now, this is the implementation that used to be in FImplicitObjectScaled to use as a default.
		// @todo(chaos): implement FindMostOpposingFaceScaled for all types that can be wrapped in ImplicitObjectScaled (Since this default implementation won't work correctly for many cases)
		// NOTE: this isn't strictly correct. The distance check does not account for non-uniforms scales, but also the algorithm is finding the
		// face which has the most opposing normal. The scaled-space normal for a face i is Ns_i = (N_i / S) / |(N_i / S)|
		// We want to find i with the minimum F_i = (D . Ns_i), where D is the unit direction we are interested in.
		// F_i = D . (N_i / S) / |(N_i / S)|
		// Below we are effectively testing F_i = ((D / S) . N_i) and ignoring the |(N_i / S)| which is dependent on i and therefore cannot 
		// be ignored in the minimum check.
		const FReal UnscaledSearchDist = SearchDist / Scale.Min();	//this is not quite right since it's no longer a sphere, but the whole thing is fuzzy anyway
		const FVec3 UnscaledPosition = Position / Scale;
		//const FVec3 UnscaledDir = ScaleNormalizedHelper(UnitDir, FVec3(1.0f / Scale.X, 1.0f / Scale.Y, 1.0f / Scale.Z)); // If we want to transform like a vector
		const FVec3 UnscaledDir = GetInnerUnscaledNormal(UnitDir, Scale); // If we want to transform like a Normal
		// This will not work correctly for most cases (e.g. if we are comparing angles (dot product) after a non conformal transformation (like non uniform scaling))
		// So really provide an implementation for FImplicitObjectScaled and avoid this default
		return FindMostOpposingFace(UnscaledPosition, UnscaledDir, HintFaceIndex, UnscaledSearchDist);
	}


	/** Finds the first intersecting face at given position
	@param Position - local position to search around (for example a point on the surface of a convex hull)
	@param FaceIndices - Vertices that lie on the face plane.
	@param SearchDistance - distance to surface [def:0.01]
	*/
	virtual int32 FindClosestFaceAndVertices(const FVec3& Position, TArray<FVec3>& FaceVertices, FReal SearchDist = 0.01f) const
	{
		//Many objects have no concept of a face
		return INDEX_NONE;
	}


	/** Given a normal and a face index, compute the most opposing normal associated with the underlying geometry features.
		For example a sphere swept against a box may not give a normal associated with one of the box faces. This function will return a normal associated with one of the faces.
		@param DenormDir - the direction we want to oppose
		@param FaceIndex - the face index associated with the geometry (for example if we hit a specific face of a convex hull)
		@param OriginalNormal - the original normal given by something like a sphere sweep
		@return The most opposing normal associated with the underlying geometry's feature (like a face)
	*/
	virtual FVec3 FindGeometryOpposingNormal(const FVec3& DenormDir, int32 FaceIndex, const FVec3& OriginalNormal) const
	{
		//Many objects have no concept of a face
		return OriginalNormal;
	}

	//This gives derived types a way to do an overlap check without calling PhiWithNormal todo: this api is confusing
	virtual bool Overlap(const FVec3& Point, const FReal Thickness) const
	{
		return SignedDistance(Point) <= Thickness;
	}

	
	virtual void AccumulateAllImplicitObjects(TArray<Pair<const FImplicitObject*, FRigidTransform3>>& Out, const FRigidTransform3& ParentTM) const
	{
		Out.Add(MakePair(this, ParentTM));
	}

	UE_DEPRECATED(5.4, "Function no longer in use")
	virtual void AccumulateAllSerializableImplicitObjects(TArray<Pair<TSerializablePtr<FImplicitObject>, FRigidTransform3>>& Out, const FRigidTransform3& ParentTM, TSerializablePtr<FImplicitObject> This) const
	{
		check(false);
	}

	CHAOS_API virtual void FindAllIntersectingObjects(TArray < Pair<const FImplicitObject*, FRigidTransform3>>& Out, const FAABB3& LocalBounds) const;

	virtual FString ToString() const
	{
		return FString::Printf(TEXT("ImplicitObject bIsConvex:%d, bDoCollide:%d, bHasBoundingBox:%d"), bIsConvex, bDoCollide, bHasBoundingBox);
	}

	CHAOS_API void SerializeImp(FArchive& Ar);

	constexpr static EImplicitObjectType StaticType()
	{
		return ImplicitObjectType::Unknown;
	}
	
	virtual void Serialize(FArchive& Ar)
	{
		check(false);	//Aggregate implicits require FChaosArchive - check false by default
	}

	CHAOS_API virtual void Serialize(FChaosArchive& Ar);
	
	static CHAOS_API FArchive& SerializeLegacyHelper(FArchive& Ar, TUniquePtr<FImplicitObject>& Value);

	virtual uint32 GetTypeHash() const = 0;

	virtual FName GetTypeName() const { return GetTypeName(GetType()); }

	static CHAOS_API const FName GetTypeName(const EImplicitObjectType InType);

	virtual uint16 GetMaterialIndex(uint32 HintIndex) const { return 0; }

	int32 CountObjectsInHierarchy() const
	{
		return CountObjectsInHierarchyImpl();
	}

	int32 CountLeafObjectsInHierarchy() const
	{
		return CountLeafObjectsInHierarchyImpl();
	}

	/**
	* Visit all the leaf objects in the hierarchy that overlap the specified local-space bounds.
	* NOTE: Templated decorators like Instanced and Scaled cound as leafs, but object decorators like Transformed do not.
	* @see FImplicitHierarchyVisitor for visitor notes
	*/
	void VisitOverlappingLeafObjects(const FAABB3& LocalBounds, const FImplicitHierarchyVisitor& Visitor) const
	{
		int32 LeafObjectIndex = 0;
		int32 ObjectIndex = 0;
		const int32 RootObjectIndex = INDEX_NONE;
		VisitOverlappingLeafObjectsImpl(LocalBounds, FRigidTransform3::Identity, RootObjectIndex, ObjectIndex, LeafObjectIndex, Visitor);
	}

	/**
	* Visit all the leaf objects in the hierarchy
	* @see FImplicitHierarchyVisitor for visitor notes
	*/
	void VisitLeafObjects(const FImplicitHierarchyVisitor& Visitor) const
	{
		int32 LeafObjectIndex = 0;
		int32 ObjectIndex = 0;
		const int32 RootObjectIndex = INDEX_NONE;
		VisitLeafObjectsImpl(FRigidTransform3::Identity, RootObjectIndex, ObjectIndex, LeafObjectIndex, Visitor);
	}

	/**
	* Visit all the objects in the hierarchy, including inner nodes like Union and Transform
	* @see FImplicitHierarchyVisitor for visitor notes
	*/
	void VisitObjects(const FImplicitHierarchyVisitorBool& Visitor) const
	{
		int32 LeafObjectIndex = 0;
		int32 ObjectIndex = 0;
		const int32 RootObjectIndex = INDEX_NONE;
		VisitObjectsImpl(FRigidTransform3::Identity, RootObjectIndex, ObjectIndex, LeafObjectIndex, Visitor);
	}

	/**
	* Whether this implicit (possibly a hierarchy) overlaps the bounds. This is a deeper, more
	* accurate test than simply checking BoundingBox() because it will query the BVH in Unions,
	* Heightfields and Triangle Meshes, and only return true if we overlap the bounds of a leaf 
	* node that contains some elements.
	*/
	bool IsOverlappingBounds(const FAABB3& LocalBounds) const
	{
		return IsOverlappingBoundsImpl(LocalBounds);
	}

//protected:
	// This should not be public, but it needs to be callable by derived classes on another instance
	virtual int32 CountObjectsInHierarchyImpl() const
	{
		return 1;
	}
	virtual int32 CountLeafObjectsInHierarchyImpl() const
	{
		return 1;
	}

	// This should not be public, but it needs to be callable by derived classes on another instance
	virtual void VisitOverlappingLeafObjectsImpl(
		const FAABB3& LocalBounds,
		const FRigidTransform3& ObjectTransform,
		const int32 RootObjectIndex,
		int32& ObjectIndex,
		int32& LeafObjectIndex,
		const FImplicitHierarchyVisitor& VisitorFunc) const
	{
		if (!HasBoundingBox() || LocalBounds.Intersects(BoundingBox()))
		{
			VisitorFunc(this, ObjectTransform, RootObjectIndex, ObjectIndex, LeafObjectIndex);
		}
		++ObjectIndex;
		++LeafObjectIndex;
	}

	// This should not be public, but it needs to be callable by derived classes on another instance
	virtual void VisitLeafObjectsImpl(
		const FRigidTransform3& ObjectTransform,
		const int32 RootObjectIndex,
		int32& ObjectIndex,
		int32& LeafObjectIndex,
		const FImplicitHierarchyVisitor& VisitorFunc) const
	{
		VisitorFunc(this, ObjectTransform, RootObjectIndex, ObjectIndex, LeafObjectIndex);
		++ObjectIndex;
		++LeafObjectIndex;
	}

	// This should not be public, but it needs to be callable by derived classes on another instance
	virtual bool VisitObjectsImpl(
		const FRigidTransform3& ObjectTransform,
		const int32 RootObjectIndex,
		int32& ObjectIndex,
		int32& LeafObjectIndex,
		const FImplicitHierarchyVisitorBool& VisitorFunc) const
	{
		const bool bResult = VisitorFunc(this, ObjectTransform, RootObjectIndex, ObjectIndex, LeafObjectIndex);
		++ObjectIndex;
		++LeafObjectIndex;
		return bResult;
	}

	// This should not be public, but it needs to be callable by derived classes on another instance
	virtual bool IsOverlappingBoundsImpl(const FAABB3& LocalBounds) const
	{
		return (!HasBoundingBox() || LocalBounds.Intersects(BoundingBox()));
	}

protected:

	// Safely scale a normalized Vec3 - used with both scale and inverse scale
	static FVec3 ScaleNormalizedHelper(const FVec3& Normal, const FVec3& Scale)
	{
		const FVec3 ScaledNormal = Scale * Normal;
		const FReal ScaledNormalLen = ScaledNormal.Size();
		return ensure(ScaledNormalLen > TNumericLimits<FReal>::Min())
			? ScaledNormal / ScaledNormalLen
			: FVec3(0.f, 0.f, 1.f);
	}

	// Convert a normal in the inner unscaled object space into a normal in the outer scaled object space.
	// Note: INverse scale is passed in, not the scale
	static FVec3 GetOuterScaledNormal(const FVec3& InnerNormal, const FVec3& Scale)
	{
		return ScaleNormalizedHelper(InnerNormal, FVec3(1.0f / Scale.X, 1.0f / Scale.Y, 1.0f / Scale.Z));
	}

	// Convert a normal in the outer scaled object space into a normal in the inner unscaled object space
	static FVec3 GetInnerUnscaledNormal(const FVec3& OuterNormal, const FVec3& Scale)
	{
		return ScaleNormalizedHelper(OuterNormal, Scale);
	}

	// Not all derived types support a margin, and for some it represents some other
	// property (the radius of a sphere for example), so the setter should only be exposed 
	// in a derived class if at all (and it may want to change the size of the core shape as well)
	void SetMargin(FReal InMargin) { Margin = static_cast<FRealSingle>(InMargin); }
	void SetMargin(FRealSingle InMargin) { Margin = InMargin; }

	// Note: if you change the size of FImplicitObject or add fields, you will likely need to update the calculation of PadBytes below.
	FRealSingle Margin;
	bool bIsConvex;
	bool bDoCollide;
	bool bHasBoundingBox;

#if TRACK_CHAOS_GEOMETRY
	bool bIsTracked;
#else
#if DISALLOW_FIMPLICIT_OBJECT_TAIL_PADDING
	// The purpose of this padding is just to make it easier to calculate PadBytes below.
	bool bPad;
#endif // DISALLOW_FIMPLICIT_OBJECT_TAIL_PADDING
#endif

	EImplicitObjectType Type;
	EImplicitObjectType CollisionType;

#if DISALLOW_FIMPLICIT_OBJECT_TAIL_PADDING
	// the following assumptions are made when calculating PadBytes and inserting any explicit padding between fields:
	static_assert(alignof(FRealSingle) <= alignof(FChaosRefCountedObject)); // Otherwise, padding will be added before Margin
	static_assert(alignof(bool) <= sizeof(FRealSingle)); // Otherwise, padding would be added between Margin the first bool field
	static_assert(alignof(EImplicitObjectType) <= 4 * sizeof(bool));  // Otherwise, padding would be added between last bool field and first EImplicitObjectType field

	static constexpr int AlignOfFImplicitObject = (int)FMath::Max(alignof(FChaosRefCountedObject), FMath::Max(alignof(FRealSingle), FMath::Max(alignof(bool), alignof(EImplicitObjectType))));
	static constexpr int PadBytes =
		AlignOfFImplicitObject - (sizeof(FChaosRefCountedObject) + sizeof(FRealSingle) + 4 * sizeof(bool) + 2 * sizeof(EImplicitObjectType)) % AlignOfFImplicitObject;
	static_assert(PadBytes > 0);
	char Pad[PadBytes];
#endif // DISALLOW_FIMPLICIT_OBJECT_TAIL_PADDING

private:
	CHAOS_API virtual Pair<FVec3, bool> FindClosestIntersectionImp(const FVec3& StartPoint, const FVec3& EndPoint, const FReal Thickness) const;
};

FORCEINLINE FChaosArchive& operator<<(FChaosArchive& Ar, FImplicitObject& Value)
{
	Value.Serialize(Ar);
	return Ar;
}

FORCEINLINE FArchive& operator<<(FArchive& Ar, FImplicitObject& Value)
{
	Value.Serialize(Ar);
	return Ar;
}

typedef FImplicitObject FImplicitObject3;
typedef TSharedPtr<Chaos::FImplicitObject, ESPMode::ThreadSafe> ThreadSafeSharedPtr_FImplicitObject;
typedef TSharedPtr<Chaos::FImplicitObject, ESPMode::NotThreadSafe> NotThreadSafeSharedPtr_FImplicitObject;
}

template <
	typename T,
	typename... TArgs
	UE_REQUIRES(!std::is_array_v<T>)
>
FORCEINLINE Chaos::FImplicitObjectPtr MakeImplicitObjectPtr(TArgs&&... Args)
{
	return Chaos::FImplicitObjectPtr(new T(Forward<TArgs>(Args)...));
}

template <
	typename T,
	typename... TArgs
	UE_REQUIRES(!std::is_array_v<T>)
>
FORCEINLINE Chaos::FConstImplicitObjectPtr MakeImplicitObjectConstPtr(TArgs&&... Args)
{
	return TRefCountPtr<const Chaos::FImplicitObject>(new T(Forward<TArgs>(Args)...));
}
