// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/BoundingVolumeHierarchy.h"
#include "Chaos/ImplicitFwd.h"
#include "Chaos/Serializable.h"

namespace Chaos
{
	class FChaosArchive;

	namespace Private
	{
		class FImplicitBVH;
		class FImplicitBVTree;
		class FImplicitBVTreeNode;

		// A item in a ImplicitBVH holding the leaf geometry and transform. Each FImplicitBVHNode node holds a set of these.
		class FImplicitBVHObject
		{
		public:
			friend class FImplicitBVH;

			friend FChaosArchive& operator<<(FChaosArchive& Ar, FImplicitBVHObject& BVHObjecty);

			FImplicitBVHObject();
			FImplicitBVHObject(const TSerializablePtr<FImplicitObject>& InGeometry, const FVec3& InX, const FRotation3& InR, const FAABB3& InBounds, const int32 InRootObjectIndex, const int32 InObjectIndex);

			const FImplicitObject* GetGeometry() const { return Geometry.GetReference(); }

			const FVec3f& GetX() const { return X; }

			FRotation3f GetR() const { return FRotation3f(FQuat4f(R[0], R[1], R[2], R[3])); }

			const FAABB3f& GetBounds() const { return Bounds; }

			const FVec3f GetBoundsCenter() const { return Bounds.Center(); }

			FRigidTransform3f GetTransformf() const { return FRigidTransform3f(GetX(), GetR()); }

			FRigidTransform3 GetTransform() const { return FRigidTransform3(FVec3(GetX()), FRotation3(GetR())); }

			// A unique index for this object in the hierarchy. E.g., if the same FImplicitObject is referenced 
			// multiple times in the hierarchy in a union of transformed objects, each will have a different ObjectIndex.
			// This is the ObjectIndex assigned when visiting the hierarchy via FImplicitObject::VisitHierachy and other 
			// visit methods and can be used to index arrays initialized via those visitors.
			// @todo(chaos): rename to GetObjectId()
			int32 GetObjectIndex() const { return ObjectIndex; }

			// The index of our most distant ancestor. I.e., the index in the root Union. This is used to map
			// each object to a ShapeInstance.
			int32 GetRootObjectIndex() const { return RootObjectIndex; }

		private:
			FChaosArchive& Serialize(FChaosArchive& Ar);

			// Transform and bounds in the space of the BVH owner (Union Implicit)
			float R[4];	//an alias for FRotation3f to avoid 16b alignment requirement that creates 12b of padding

			FVec3f X;
			FAABB3f Bounds;

			// The index of our ancestor in the array of RootObjects that was provided when creating the BVH
			int32 RootObjectIndex;

			// The leaf geometry stripped of decorators (but not Instanced or Scaled)
			TSerializablePtr<FImplicitObject> Geometry;

			// Our index in the hierarchy. This could be used to uniquely identity copies of the same implicit in the hierarchy
			int32 ObjectIndex;
		};

		// A node in an FImplicitBVH
		class FImplicitBVHNode
		{
		public:
			FImplicitBVHNode(const FAABB3f& InBounds, const int32 InObjectBeginIndex, const int32 InObjectEndIndex)
				: Bounds(InBounds)
				, ObjectBeginIndex(InObjectBeginIndex)
				, ObjectEndIndex(InObjectEndIndex)
				, ChildNodeIndices{ INDEX_NONE, INDEX_NONE }
			{
			}

			bool IsLeaf() const
			{
				// NOTE: We either have zero or two children
				return ChildNodeIndices[0] == INDEX_NONE;
			}

		private:
			friend class FImplicitBVH;

			FAABB3f Bounds;
			int32 ObjectBeginIndex;
			int32 ObjectEndIndex;
			int32 ChildNodeIndices[2];
		};


		// A Bounding Volume Hierarchy of a set of Implicit Objects
		class FImplicitBVH
		{
		public:
			using FObjects = TArray<FImplicitBVHObject>;

			friend FChaosArchive& operator<<(FChaosArchive& Ar, FImplicitBVH& BVH);

			// Make an empty BVH (for serialization only)
			static TUniquePtr<FImplicitBVH> MakeEmpty();

			// Utility for processing the hierarchy
			static int32 CountLeafObjects(const TArrayView<const Chaos::FImplicitObjectPtr>& InRootObjects);
			static FObjects CollectLeafObjects(const TArrayView<const Chaos::FImplicitObjectPtr>& InRootObjects);
			static void CollectLeafObject(const FImplicitObject* Object, const FRigidTransform3& ParentTransform, const int32 RootObjectIndex,
				TArray<FImplicitBVHObject>& LeafObjects, const int32 LeafObjectIndex);

			// Create a BVH around a set of ImplicitObjects. Usually these are the immediate child elements of an FImplcitObjectUnion
			// TryMake will then recurse into the geometry hierachy and add all descendents to the BVH. Will return null if the 
			// number of descendents is less that MinObjects.
			static TUniquePtr<FImplicitBVH> TryMake(const TArrayView<const Chaos::FImplicitObjectPtr>& InRootObjects, const int32 MinObjects, const int32 InMaxBVHDepth);
			
			// Create a BVH around given a list of leaves.  Will return null if the  number of descendents is less that MinObjects.
			static TUniquePtr<FImplicitBVH> TryMakeFromLeaves(TArray<FImplicitBVHObject>&& LeafObjects, const int32 InMinObjects, const int32 InMaxBVHDepth);

			~FImplicitBVH();

			int32 GetNumObjects() const { return Objects.Num(); }
			int32 GetDepth() const { return TreeDepth; }
			
			const TArray<FImplicitBVHObject>& GetObjects() const { return Objects; }
			
			const FImplicitBVHObject& GetObject(const int32 ObjectIndex) const { return Objects[ObjectIndex]; }

			const FImplicitObject* GetGeometry(const int32 ObjectIndex) const { return Objects[ObjectIndex].GetGeometry(); }

			const FVec3f& GetX(const int32 ObjectIndex) const { return Objects[ObjectIndex].GetX(); }

			FRotation3f GetR(const int32 ObjectIndex) const { return Objects[ObjectIndex].GetR(); }

			const FAABB3f& GetBounds(const int32 ObjectIndex) const { return Objects[ObjectIndex].GetBounds(); }

			FRigidTransform3f GetTransformf(const int32 ObjectIndex) const { return Objects[ObjectIndex].GetTransformf(); }

			FRigidTransform3 GetTransform(const int32 ObjectIndex) const { return Objects[ObjectIndex].GetTransform(); }

			int32 GetRootObjectIndex(const int32 ObjectIndex) const { return Objects[ObjectIndex].GetRootObjectIndex(); }
			int32 GetObjectIndex(const int32 ObjectIndex) const { return Objects[ObjectIndex].GetObjectIndex(); }

			// Visit all the leaf objects that overlap the specified bounds.
			// @param ObjectVisitor [](const FImplicitObject* Implicit, const FRigidTransform3f& RelativeTransformf, const FAABB3f& RelativeBoundsf, const int32 RootObjectIndex, const int32 LeafObjectIndex) -> void {}
			template<typename TVisitor>
			void VisitAllIntersections(const FAABB3& LocalBounds, const TVisitor& ObjectVisitor) const
			{
				const auto& NodeVisitor = [this, &ObjectVisitor](const int32 NodeIndex)
				{
					if (NodeIsLeaf(NodeIndex))
					{
						VisitNodeObjects(NodeIndex, ObjectVisitor);
					}
				};

				VisitOverlappingNodesStack(FAABB3f(LocalBounds), NodeVisitor);
			}

			// Calls the visitor for every overlapping leaf.
			// @tparam TVisitor void(const int32 NodeIndex)
			template<typename TVisitor>
			void VisitOverlappingNodes(const FAABB3& LocalBounds, const TVisitor& NodeVisitor) const
			{
				VisitOverlappingNodesStack(FAABB3f(LocalBounds), NodeVisitor);
			}

			// Recursively visit all nodes in the hierarchy. Will stop visiting children
			// if the visitor returns false. Leaf will be null when visiting an internal node.
			// @param NodeVisitor (const FAABB3f& NodeBounds, const int32 NodeDepth, const int32 NodeIndex) -> void
			template<typename TVisitor>
			void VisitNodes(const TVisitor& NodeVisitor) const
			{
				VisitNodesStack(NodeVisitor);
			}


			// Visit all the items in the specified leaf node (which is probably obtained from VisitHierarchy)
			// @param ObjectVisitor (const FImplicitObject* Implicit, const FRigidTransform3f& RelativeTransformf, const FAABB3f& RelativeBoundsf, const int32 RootObjectIndex, const int32 LeafObjectIndex) -> void
			template<typename TVisitor>
			void VisitNodeObjects(const int32 NodeIndex, const TVisitor& ObjectVisitor) const
			{
				const FImplicitBVHNode& Node = Nodes[NodeIndex];

				for (int32 NodeObjectIndex = Node.ObjectBeginIndex; NodeObjectIndex < Node.ObjectEndIndex; ++NodeObjectIndex)
				{
					const FImplicitBVHObject& Object = Objects[NodeObjectIndices[NodeObjectIndex]];

					ObjectVisitor(Object.GetGeometry(), Object.GetTransformf(), Object.GetBounds(), Object.GetRootObjectIndex(), Object.GetObjectIndex());
				}
			}

			// Does the bounding box overlap any leaf nodes with items in it
			bool IsOverlappingBounds(const FAABB3& LocalBounds) const
			{
				return IsOverlappingBoundsStack(FAABB3f(LocalBounds));
			}

			template<typename TVisitor>
			static void VisitOverlappingLeafNodes(const FImplicitBVH& BVHA, const FImplicitBVH& BVHB, const FRigidTransform3& TransformBToA, const TVisitor& LeafPairVisitor)
			{
				VisitOverlappingLeafNodesStack(BVHA, BVHB, TransformBToA, LeafPairVisitor);
			}

			int32 GetNumNodes() const
			{
				return Nodes.Num();
			}

			const FAABB3f& GetNodeBounds(const int32 NodeIndex) const
			{
				return Nodes[NodeIndex].Bounds;
			}

			bool NodeIsLeaf(const int32 NodeIndex) const
			{
				return Nodes[NodeIndex].IsLeaf();
			}

		private:
			FImplicitBVH();
			FImplicitBVH(FObjects&& InObjects, const int32 InMaxBVHDepth);
			FChaosArchive& Serialize(FChaosArchive& Ar);

			// Initialize the BVH from the specified set of children.
			// NOTE: InChildren should be immediate children of the BVH owner, not further-removed descendents
			void Init(FObjects&& InObjects, const int32 MaxDepth, const int32 MaxLeafObjects);

			template<typename TVisitor>
			void VisitOverlappingNodesStack(const FAABB3f& LocalBounds, const TVisitor& NodeVisitor) const
			{
				if (Nodes.IsEmpty())
				{
					return;
				}

				FMemMark Mark(FMemStack::Get());
				TArray<int32, TMemStackAllocator<>> NodeStack;
				NodeStack.Reserve(GetDepth());

				int32 NodeIndex = 0;
				while (true)
				{
					const FImplicitBVHNode& Node = Nodes[NodeIndex];

					if (Node.Bounds.Intersects(LocalBounds))
					{
						if (Node.IsLeaf())
						{
							NodeVisitor(NodeIndex);
						}
						else
						{
							check(NodeStack.Num() < GetDepth());

							NodeIndex = Node.ChildNodeIndices[0];
							NodeStack.Push(Node.ChildNodeIndices[1]);
							continue;
						}
					}

					if (NodeStack.IsEmpty())
					{
						break;
					}

					NodeIndex = NodeStack.Pop(EAllowShrinking::No);
				}
			}

			template<typename TVisitor>
			void VisitNodesStack(const TVisitor& NodeVisitor) const
			{
				if (Nodes.IsEmpty())
				{
					return;
				}

				FMemMark Mark(FMemStack::Get());
				TArray<TPair<int32, int32>, TMemStackAllocator<>> NodeStack;	// NodeIndex, NodeDepth
				NodeStack.Reserve(GetDepth());

				int32 NodeIndex = 0;
				int32 NodeDepth = 0;
				while (true)
				{
					const FImplicitBVHNode& Node = Nodes[NodeIndex];

					const bool bVisitChildren = NodeVisitor(Node.Bounds, NodeDepth, NodeIndex);

					if (bVisitChildren && !Node.IsLeaf())
					{
						check(NodeStack.Num() < GetDepth());

						const int32 ChildNodeIndexL = Node.ChildNodeIndices[0];
						const int32 ChildNodeIndexR = Node.ChildNodeIndices[1];
						NodeDepth = NodeDepth + 1;
						NodeIndex = ChildNodeIndexL;
						NodeStack.Push({ ChildNodeIndexR, NodeDepth });
						continue;
					}

					if (NodeStack.IsEmpty())
					{
						break;
					}

					NodeIndex = NodeStack.Top().Key;
					NodeDepth = NodeStack.Top().Value;
					NodeStack.Pop(EAllowShrinking::No);
				}
			}

			template<typename TVisitor>
			static void VisitOverlappingLeafNodesStack(const FImplicitBVH& BVHA, const FImplicitBVH& BVHB, const FRigidTransform3& TransformBToA, const TVisitor& LeafPairVisitor)
			{
				if (BVHA.Nodes.IsEmpty() || BVHB.Nodes.IsEmpty())
				{
					return;
				}

				// The node pair stack
				FMemMark Mark(FMemStack::Get());
				TArray<TVec2<int32>, TMemStackAllocator<>> NodePairStack;
				const int32 NodeStackMax = BVHA.GetDepth() + BVHB.GetDepth();
				NodePairStack.Reserve(NodeStackMax);

				int32 NodeIndexA = 0;
				int32 NodeIndexB = 0;

				while (true)
				{
					const FImplicitBVHNode& NodeA = BVHA.Nodes[NodeIndexA];
					const FImplicitBVHNode& NodeB = BVHB.Nodes[NodeIndexB];

					const FAABB3f& BoundsA = NodeA.Bounds;
					FAABB3f BoundsBInA = NodeB.Bounds.TransformedAABB(TransformBToA);

					if (BoundsA.Intersects(BoundsBInA))
					{
						if (NodeA.IsLeaf() && NodeB.IsLeaf())
						{
							LeafPairVisitor(NodeIndexA, NodeIndexB);
						}
						else
						{
							check(NodePairStack.Num() < NodeStackMax);

							// @todo(chaos): rule to choose whether to descend into A or B first
							// Descend into B first, until we reach its leaf nodes
							const bool bDescendA = NodeB.IsLeaf();
							if (bDescendA)
							{
								// Descend A
								NodeIndexA = NodeA.ChildNodeIndices[0];
								NodePairStack.Push({ NodeA.ChildNodeIndices[1], NodeIndexB });
								continue;
							}
							else
							{
								// Descend B
								NodeIndexB = NodeB.ChildNodeIndices[0];
								NodePairStack.Push({ NodeIndexA, NodeB.ChildNodeIndices[1] });
								continue;
							}
						}
					}

					// If we get here we just processed a leaf or did not overlap,
					// and need to pop an item off the stack to continue
					if (NodePairStack.IsEmpty())
					{
						break;
					}
					NodeIndexA = NodePairStack.Top()[0];
					NodeIndexB = NodePairStack.Top()[1];
					NodePairStack.Pop(EAllowShrinking::No);
				}
			}

			bool IsOverlappingBoundsStack(const FAABB3f& LocalBounds) const
			{
				if (Nodes.IsEmpty())
				{
					return false;
				}

				FMemMark Mark(FMemStack::Get());
				TArray<int32, TMemStackAllocator<>> NodeStack;
				NodeStack.Reserve(GetDepth());

				int32 NodeIndex = 0;
				while (true)
				{
					const FImplicitBVHNode& Node = Nodes[NodeIndex];

					if (Node.Bounds.Intersects(LocalBounds))
					{
						if (Node.IsLeaf())
						{
							return true;
						}
						check(NodeStack.Num() < GetDepth());

						NodeIndex = Node.ChildNodeIndices[0];
						NodeStack.Push(Node.ChildNodeIndices[1]);
						continue;
					}

					if (NodeStack.IsEmpty())
					{
						break;
					}
					NodeIndex = NodeStack.Pop(EAllowShrinking::No);
				}

				return false;
			}

			struct FWorkData
			{
				TArray<FVec3f> ObjectCenters;
				TArray<int32> NodeObjectIndices0;
				TArray<int32> NodeObjectIndices1;
			};

			void BuildTree(const int32 MaxDepth, const int32 MaxLeafObjects);
			int32 AddNodeRecursive(const FAABB3f& NodeBounds, const int32 ObjectBeginIndex, const int32 ObjectEndIndex, const int32 NodeDepth, const int32 MaxDepth, const int32 MaxLeafObjects, FWorkData& WorkData);
			bool Partition(const FAABB3f& NodeBounds, const int32 ObjectBeginIndex, const int32 ObjectEndIndex, FAABB3f& OutChildNodeBounds0, FAABB3f& OutChildNodeBounds1, int32& OutPartitionIndex, FWorkData& WorkData);

			// A BVH leaf holds an array of indices into the Objects array
			FObjects Objects;
			TArray<int32> NodeObjectIndices;
			TArray<FImplicitBVHNode> Nodes;
			int32 TreeDepth;
		};
	}

	// DO NOT USE: This is only required for serializing the old BVH data
	template<> inline bool HasBoundingBox(const TArray<Private::FImplicitBVHObject>& Objects, const int32 ObjectIndex) { return Objects[ObjectIndex].GetGeometry()->HasBoundingBox(); }
	template<class T, int d> inline const TAABB<T, d> GetWorldSpaceBoundingBox(const TArray<Private::FImplicitBVHObject>& Objects, const int32 ObjectIndex, const TMap<int32, TAABB<T, d>>& WorldSpaceBoxes) { return TAABB<T, d>(Objects[ObjectIndex].GetBounds()); }
	template<class T, int d> void ComputeAllWorldSpaceBoundingBoxes(const TArray<Private::FImplicitBVHObject>& Objects, const TArray<int32>& AllObjects, const bool bUseVelocity, const T Dt, TMap<int32, TAABB<T, d>>& WorldSpaceBoxes) { }
}