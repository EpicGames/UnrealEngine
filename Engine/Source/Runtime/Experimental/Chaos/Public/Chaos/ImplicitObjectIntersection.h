// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Array.h"
#include "Chaos/ImplicitObject.h"

#include <memory>

namespace Chaos
{
template<class T, int d>
class TImplicitObjectIntersection : public FImplicitObject
{
  public:
	using FImplicitObject::SignedDistance;

	TImplicitObjectIntersection(TArray<Chaos::FImplicitObjectPtr>&& Objects)
	    : FImplicitObject(EImplicitObject::HasBoundingBox)
	    , MObjects(MoveTemp(Objects))
	    , MLocalBoundingBox(MObjects[0]->BoundingBox())
	{
		for (int32 i = 1; i < MObjects.Num(); ++i)
		{
			MLocalBoundingBox.ShrinkToInclude(MObjects[i]->BoundingBox());
		}
	}
	
	UE_DEPRECATED(5.4, "Use TImplicitObjectIntersection constructor with FImplicitObjectPtr instead")
	TImplicitObjectIntersection(TArray<TUniquePtr<FImplicitObject>>&& Objects)
		: FImplicitObject(EImplicitObject::HasBoundingBox)
		, MObjects()
		, MLocalBoundingBox(MObjects[0]->BoundingBox())
	{
		check(false);
	}
	
	TImplicitObjectIntersection(const TImplicitObjectIntersection<T, d>& Other) = delete;
	TImplicitObjectIntersection(TImplicitObjectIntersection<T, d>&& Other)
	    : FImplicitObject(EImplicitObject::HasBoundingBox)
	    , MObjects(MoveTemp(Other.MObjects))
	    , MLocalBoundingBox(MoveTemp(Other.MLocalBoundingBox))
	{
	}
	virtual ~TImplicitObjectIntersection() = default;

	virtual T PhiWithNormal(const TVector<T, d>& x, TVector<T, d>& Normal) const override
	{
		check(MObjects.Num());
		T Phi = MObjects[0]->PhiWithNormal(x, Normal);
		for (int32 i = 1; i < MObjects.Num(); ++i)
		{
			TVector<T, d> NextNormal;
			T NextPhi = MObjects[i]->PhiWithNormal(x, NextNormal);
			if (NextPhi > Phi)
			{
				Phi = NextPhi;
				Normal = NextNormal;
			}
			else if (NextPhi == Phi)
			{
				Normal += NextNormal;
			}
		}
		Normal.Normalize();
		return Phi;
	}

	virtual const TAABB<T,d> BoundingBox() const { return MLocalBoundingBox; }


	virtual uint32 GetTypeHash() const override
	{
		uint32 OutHash = MObjects.Num() > 0 ? MObjects[0]->GetTypeHash() : 0;

		for(const Chaos::FImplicitObjectPtr& Ptr : MObjects)
		{
			OutHash = HashCombine(Ptr->GetTypeHash(), OutHash);
		}

		return OutHash;
	}

private:
	virtual Pair<TVector<T, d>, bool> FindClosestIntersectionImp(const TVector<T, d>& StartPoint, const TVector<T, d>& EndPoint, const T Thickness) const
	{
		TArray<Pair<T, TVector<T, d>>> Intersections;
		for (int32 i = 0; i < MObjects.Num(); ++i)
		{
			auto NextClosestIntersection = MObjects[i]->FindClosestIntersection(StartPoint, EndPoint, Thickness);
			if (!NextClosestIntersection.Second)
				continue;
			Intersections.Add(MakePair((FReal)(NextClosestIntersection.First - StartPoint).Size(), NextClosestIntersection.First));
		}
		Intersections.Sort([](const Pair<T, TVector<T, d>>& Elem1, const Pair<T, TVector<T, d>>& Elem2) { return Elem1.First < Elem2.First; });
		for (int32 i = 0; i < Intersections.Num(); ++i)
		{
			if (SignedDistance(Intersections[i].Second) <= (Thickness + 1e-4))
			{
				return MakePair(Intersections[i].Second, true);
			}
		}
		return MakePair(TVector<T, d>(0), false);
	}

  private:
	TArray<Chaos::FImplicitObjectPtr> MObjects;
	TAABB<T, d> MLocalBoundingBox;
};
}
