// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Box.h"
#include "Chaos/Collision/ContactPoint.h"
#include "Chaos/Convex.h"
#include "Chaos/ImplicitFwd.h"
#include "Chaos/ImplicitObject.h"
#include "Chaos/Transform.h"
#include "Chaos/Utilities.h"	// For ScaleInertia - pull that into mass utils
#include "ChaosArchive.h"
#include "Templates/EnableIf.h"
#include "Math/NumericLimits.h"
#include "ChaosCheck.h"

namespace Chaos
{
struct FMTDInfo;

namespace Rigids::Async
{
	class FRigidShapeInstanceAsync;
} // namespace Rigids::Async

class FImplicitObjectInstanced : public FImplicitObject
{
public:
	FImplicitObjectInstanced(int32 Flags, EImplicitObjectType InType)
		: FImplicitObject(Flags, InType | ImplicitObjectType::IsInstanced)
		, OuterMargin(0)
	{
	}

	virtual TSerializablePtr<FImplicitObject> GetInnerObject() const
	{
		return TSerializablePtr<FImplicitObject>();
	}

	// Returns a winding order multiplier used in the manifold clipping and required when we have negative scales
	FORCEINLINE FReal GetWindingOrder() const
	{
		return 1.0f;
	}


protected:
	friend class Rigids::Async::FRigidShapeInstanceAsync;
	UE_INTERNAL virtual FImplicitObjectPtr GetInnerObjectInternal() const
	{
		return FImplicitObjectPtr();
	}

	FRealSingle OuterMargin;
};

template <typename TConcrete>
class TImplicitObjectInstanced final : public FImplicitObjectInstanced
{
public:
	using T = typename TConcrete::TType;
	using TType = T;
	static constexpr int d = TConcrete::D;
	static constexpr int D = d;
	using ObjectType = TRefCountPtr<TConcrete>;

	using FImplicitObject::GetTypeName;

	//needed for serialization
	TImplicitObjectInstanced()
		: FImplicitObjectInstanced(EImplicitObject::HasBoundingBox,StaticType())
	{
		this->OuterMargin = 0;
	}

	TImplicitObjectInstanced(const ObjectType&& Object, const FReal InMargin = 0)
		: FImplicitObjectInstanced(EImplicitObject::HasBoundingBox, Object->GetType())
		, MObject(MoveTemp(Object))
	{
		ensure(IsInstanced(MObject->GetType()) == false);	//cannot have an instance of an instance
		this->bIsConvex = MObject->IsConvex();
		this->bDoCollide = MObject->GetDoCollide();
		this->OuterMargin = FRealSingle(InMargin);
		SetMargin(OuterMargin + MObject->GetMarginf());
	}

	TImplicitObjectInstanced(const ObjectType& Object, const FReal InMargin = 0)
		: FImplicitObjectInstanced(EImplicitObject::HasBoundingBox,Object->GetType())
		, MObject(Object)
	{
		ensure(IsInstanced(MObject->GetType()) == false);	//cannot have an instance of an instance
		this->bIsConvex = MObject->IsConvex();
		this->bDoCollide = MObject->GetDoCollide();
		this->OuterMargin = FRealSingle(InMargin);
		SetMargin(OuterMargin + MObject->GetMarginf());
	}

	TImplicitObjectInstanced(TImplicitObjectInstanced<TConcrete>&& Other)
		: FImplicitObjectInstanced(EImplicitObject::HasBoundingBox, Other.MObject->GetType())
		, MObject(MoveTemp(Other.MObject))
	{
		ensureMsgf((IsScaled(MObject->GetType()) == false), TEXT("Scaled objects should not contain each other."));
		ensureMsgf((IsInstanced(MObject->GetType()) == false), TEXT("Scaled objects should not contain instances."));
		this->bIsConvex = MObject->IsConvex();
		this->bDoCollide = MObject->GetDoCollide();
		this->OuterMargin = Other.OuterMargin;
		SetMargin(Other.GetMarginf());
	}

	static constexpr EImplicitObjectType StaticType()
	{
		return TConcrete::StaticType() | ImplicitObjectType::IsInstanced;
	}

	virtual TSerializablePtr<FImplicitObject> GetInnerObject() const override
	{
		return MakeSerializable(MObject);
	}

	const TConcrete* GetInstancedObject() const
	{
		return MObject.GetReference();
	}

	UE_DEPRECATED(5.6, "Use GetRadiusf instead")
	virtual FReal GetRadius() const override
	{
		return MObject->GetRadiusf();
	}

	virtual FRealSingle GetRadiusf() const override
	{
		return MObject->GetRadiusf();
	}

	bool GetDoCollide() const
	{
		return MObject->GetDoCollide();
	}

	virtual FReal PhiWithNormal(const FVec3& X, FVec3& Normal) const override
	{
		return MObject->PhiWithNormal(X, Normal);
	}

	virtual bool Raycast(const FVec3& StartPoint, const FVec3& Dir, const FReal Length, const FReal Thickness, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex) const override
	{
		return MObject->Raycast(StartPoint, Dir, Length, Thickness, OutTime, OutPosition, OutNormal, OutFaceIndex);
	}

	virtual void Serialize(FChaosArchive& Ar) override
	{
		FChaosArchiveScopedMemory ScopedMemory(Ar, GetTypeName(), false);
		FImplicitObject::SerializeImp(Ar);
		Ar << MObject;
	}

	virtual int32 FindMostOpposingFace(const FVec3& Position, const FVec3& UnitDir, int32 HintFaceIndex, FReal SearchDist) const override
	{
		return MObject->FindMostOpposingFace(Position, UnitDir, HintFaceIndex, SearchDist);
	}

	virtual FVec3 FindGeometryOpposingNormal(const FVec3& DenormDir, int32 HintFaceIndex, const FVec3& OriginalNormal) const override
	{
		return MObject->FindGeometryOpposingNormal(DenormDir, HintFaceIndex, OriginalNormal);
	}

	virtual bool Overlap(const FVec3& Point, const FReal Thickness) const override
	{
		return MObject->Overlap(Point, Thickness);
	}

	// The support position from the specified direction
	FORCEINLINE_DEBUGGABLE FVec3 Support(const FVec3& Direction, const FReal Thickness, int32& VertexIndex) const
	{
		return MObject->Support(Direction, Thickness, VertexIndex); 
	}

	// this shouldn't be called, but is required until we remove the explicit function implementations in CollisionResolution.cpp
	FORCEINLINE_DEBUGGABLE FVec3 SupportScaled(const FVec3& Direction, const T Thickness, const FVec3& Scale, int32& VertexIndex) const
	{
		return MObject->SupportScaled(Direction, Thickness, Scale, VertexIndex);
	}

	// The support position from the specified direction, if the shape is reduced by the margin
	FORCEINLINE_DEBUGGABLE FVec3 SupportCore(const FVec3& Direction, const FReal InMargin, FReal* OutSupportDelta, int32& VertexIndex) const
	{
		return MObject->SupportCore(Direction, InMargin, OutSupportDelta, VertexIndex);
	}

	FORCEINLINE_DEBUGGABLE VectorRegister4Float SupportCoreSimd(const VectorRegister4Float& Direction, const FReal InMargin) const
	{
		FVec3 DirectionVec3;
		VectorStoreFloat3(Direction, &DirectionVec3);
		int32 VertexIndex = INDEX_NONE;
		FVec3 SupportVert = MObject->SupportCore(DirectionVec3, InMargin, nullptr, VertexIndex);
		return MakeVectorRegisterFloatFromDouble(MakeVectorRegister(SupportVert.X, SupportVert.Y, SupportVert.Z, 0.0));
	}

	virtual const FAABB3 BoundingBox() const override 
	{ 
		return MObject->BoundingBox();
	}

	const ObjectType Object() const { return MObject; }

	virtual uint32 GetTypeHash() const override
	{
		return MObject->GetTypeHash();
	}

	virtual EImplicitObjectType GetNestedType() const override
	{
		return MObject->GetNestedType();
	}

	virtual Chaos::FImplicitObjectPtr CopyGeometry() const override
	{
		return Chaos::FImplicitObjectPtr(CopyHelper(this));
	}

	virtual Chaos::FImplicitObjectPtr CopyGeometryWithScale(const FVec3& Scale) const override;

	virtual FString ToString() const override
	{
		return FString::Printf(TEXT("Instanced %s, Margin %f"), *MObject->ToString(), GetMarginf());
	}

	static const TImplicitObjectInstanced<TConcrete>& AsInstancedChecked(const FImplicitObject& Obj)
	{
		if constexpr (std::is_same_v<TConcrete,FImplicitObject>)
		{
			//can cast any instanced to ImplicitObject base
			check(IsInstanced(Obj.GetType()));
		}
		else
		{
			check(StaticType() == Obj.GetType());
		}
		return static_cast<const TImplicitObjectInstanced<TConcrete>&>(Obj);
	}
	
	static const TImplicitObjectInstanced<TConcrete>* AsInstanced(const FImplicitObject& Obj)
	{
		if constexpr (std::is_same_v<TConcrete, FImplicitObject>)
		{
			//can cast any scaled to ImplicitObject base
			return IsInstanced(Obj.GetType()) ? static_cast<const TImplicitObjectInstanced<TConcrete>*>(&Obj) : nullptr;
		}
		else
		{
			return StaticType() == Obj.GetType() ? static_cast<const TImplicitObjectInstanced<TConcrete>*>(&Obj) : nullptr;
		}
	}

	static TImplicitObjectInstanced<TConcrete>* AsInstanced(FImplicitObject& Obj)
	{
		if constexpr (std::is_same_v<TConcrete, FImplicitObject>)
		{
			//can cast any scaled to ImplicitObject base
			return IsInstanced(Obj.GetType()) ? static_cast<TImplicitObjectInstanced<TConcrete>*>(&Obj) : nullptr;
		}
		else
		{
			return StaticType() == Obj.GetType() ? static_cast<TImplicitObjectInstanced<TConcrete>*>(&Obj) : nullptr;
		}
	}

	/** This is a low level function and assumes the internal object has a SweepGeom function. Should not be called directly. See GeometryQueries.h : SweepQuery */
	template <typename QueryGeomType>
	bool LowLevelSweepGeom(const QueryGeomType& B, const TRigidTransform<T, d>& BToATM, const TVector<T, d>& LocalDir, const T Length, T& OutTime, TVector<T, d>& LocalPosition, TVector<T, d>& LocalNormal, int32& OutFaceIndex, TVector<T, d>& OutFaceNormal, T Thickness = 0, bool bComputeMTD = false) const
	{
		return MObject->SweepGeom(B, BToATM, LocalDir, Length, OutTime, LocalPosition, LocalNormal, OutFaceIndex, OutFaceNormal, Thickness, bComputeMTD);
	}

	/** This is a low level function and assumes the internal object has a OverlapGeom function. Should not be called directly. See GeometryQueries.h : OverlapQuery */
	template <typename QueryGeomType>
	bool LowLevelOverlapGeom(const QueryGeomType& B,const TRigidTransform<T,d>& BToATM,T Thickness = 0, FMTDInfo* OutMTD = nullptr) const
	{
		return MObject->OverlapGeom(B,BToATM,Thickness, OutMTD);
	}

	template <typename QueryGeomType>
	bool GJKContactPoint(const QueryGeomType& A, const FRigidTransform3& AToBTM, const FReal Thickness, FVec3& Location, FVec3& Normal, FReal& Penetration, int32& FaceIndex) const
	{
		return MObject->GJKContactPoint(A, AToBTM, Thickness, Location, Normal, Penetration, FaceIndex);
	}

	virtual uint16 GetMaterialIndex(uint32 HintIndex) const
	{
		return MObject->GetMaterialIndex(HintIndex);
	}

	// Get the index of the plane that most opposes the normal
	int32 GetMostOpposingPlane(const FVec3& Normal) const
	{
		return MObject->GetMostOpposingPlane(Normal);
	}

	// Get the nearest point on an edge of the specified face
	FVec3 GetClosestEdge(int32 PlaneIndex, const FVec3& Position, FVec3& OutEdgePos0, FVec3& OutEdgePos1) const
	{
		return MObject->GetClosestEdge(PlaneIndex, Position, OutEdgePos0, OutEdgePos1);
	}

	// Get the nearest point on an edge of the specified face
	FVec3 GetClosestEdgePosition(int32 PlaneIndex, const FVec3& Position) const
	{
		return MObject->GetClosestEdgePosition(PlaneIndex, Position);
	}

	bool GetClosestEdgeVertices(int32 PlaneIndexHint, const FVec3& Position, int32& OutVertexIndex0, int32& OutVertexIndex1) const
	{
		return MObject->GetClosestEdgeVertices(PlaneIndexHint, Position, OutVertexIndex0, OutVertexIndex1);
	}

	// Get an array of all the plane indices that belong to a vertex (up to MaxVertexPlanes).
	// Returns the number of planes found.
	int32 FindVertexPlanes(int32 VertexIndex, int32* OutVertexPlanes, int32 MaxVertexPlanes) const
	{
		return MObject->FindVertexPlanes(VertexIndex, OutVertexPlanes, MaxVertexPlanes);
	}

	// Get up to the 3  plane indices that belong to a vertex
	// Returns the number of planes found.
	int32 GetVertexPlanes3(int32 VertexIndex, int32& PlaneIndex0, int32& PlaneIndex1, int32& PlaneIndex2) const
	{
		return MObject->GetVertexPlanes3(VertexIndex, PlaneIndex0, PlaneIndex1, PlaneIndex2);
	}

	// The number of vertices that make up the corners of the specified face
	int32 NumPlaneVertices(int32 PlaneIndex) const
	{
		return MObject->NumPlaneVertices(PlaneIndex);
	}

	// Get the vertex index of one of the vertices making up the corners of the specified face
	int32 GetPlaneVertex(int32 PlaneIndex, int32 PlaneVertexIndex) const
	{
		return MObject->GetPlaneVertex(PlaneIndex, PlaneVertexIndex);
	}

	int32 GetEdgeVertex(int32 EdgeIndex, int32 EdgeVertexIndex) const
	{
		return MObject->GetEdgeVertex(EdgeIndex, EdgeVertexIndex);
	}

	int32 GetEdgePlane(int32 EdgeIndex, int32 EdgePlaneIndex) const
	{
		return MObject->GetEdgePlane(EdgeIndex, EdgePlaneIndex);
	}

	int32 NumPlanes() const
	{
		return MObject->NumPlanes();
	}

	int32 NumEdges() const
	{
		return MObject->NumEdges();
	}

	int32 NumVertices() const
	{
		return MObject->NumVertices();
	}

	// Get the plane at the specified index (e.g., indices from FindVertexPlanes)
	const TPlaneConcrete<FReal, 3> GetPlane(int32 FaceIndex) const
	{
		return MObject->GetPlane(FaceIndex);
	}

	void GetPlaneNX(const int32 FaceIndex, FVec3& OutN, FVec3& OutX) const
	{
		MObject->GetPlaneNX(FaceIndex, OutN, OutX);
	}

	// Get the vertex at the specified index (e.g., indices from GetPlaneVertexs)
	const FVec3 GetVertex(int32 VertexIndex) const
	{
		return MObject->GetVertex(VertexIndex);
	}

	const FVec3 GetCenterOfMass() const
	{
		return MObject->GetCenterOfMass();
	}

	FRotation3 GetRotationOfMass() const
	{
		return GetRotationOfMass();
	}

	const FMatrix33 GetInertiaTensor(const FReal Mass) const
	{
		return MObject->GetInertiaTensor(Mass);
	}

protected:
	ObjectType MObject;

	static TImplicitObjectInstanced<TConcrete>* CopyHelper(const TImplicitObjectInstanced<TConcrete>* Obj)
	{
		return new TImplicitObjectInstanced<TConcrete>(Obj->MObject);
	}

private:	
	UE_INTERNAL virtual FImplicitObjectPtr GetInnerObjectInternal() const override final
	{
		return FImplicitObjectPtr(MObject.GetReference());
	}
};

class FImplicitObjectScaled : public FImplicitObject
{
public:
	FImplicitObjectScaled(int32 Flags, EImplicitObjectType InType)
		: FImplicitObject(Flags, InType | ImplicitObjectType::IsScaled)
		, MScale(1)
		, MInvScale(1)
		, OuterMargin(0.0f)
		, MLocalBoundingBox(FAABB3::EmptyAABB())
	{
	}

	virtual TSerializablePtr<FImplicitObject> GetInnerObject() const
	{
		return TSerializablePtr<FImplicitObject>();
	}

	// Returns a winding order multiplier used in the manifold clipping and required when we have negative scales
	FORCEINLINE FReal GetWindingOrder() const
	{
		const FVec3 SignVector = MScale.GetSignVector();
		return SignVector.X * SignVector.Y * SignVector.Z;
	}

	// UE_DEPRECATED(5.6, "Use GetScalef instead")
	const FVec3 GetScale() const
	{
		return FVec3(MScale);
	}

	// UE_DEPRECATED(5.6, "Use GetInvScalef instead")
	const FVec3 GetInvScale() const
	{
		return FVec3(MInvScale);
	}

	virtual const FAABB3 BoundingBox() const override
	{
		return MLocalBoundingBox;
	}

protected:
	friend class Rigids::Async::FRigidShapeInstanceAsync;
	UE_INTERNAL virtual FImplicitObjectPtr GetInnerObjectInternal() const
	{
		return FImplicitObjectPtr();
	}

	FVec3f MScale;
	FVec3f MInvScale;
	FRealSingle OuterMargin;	//Allows us to inflate the instance before the scale is applied. This is useful when sweeps need to apply a non scale on a geometry with uniform thickness
	FAABB3 MLocalBoundingBox;
};

template<typename TConcrete, bool bInstanced = true>
class TImplicitObjectScaled final : public FImplicitObjectScaled
{
public:
	using T = typename TConcrete::TType;
	using TType = T;
	static constexpr int d = TConcrete::D;
	static constexpr int D = d;

	using ObjectType = TRefCountPtr<TConcrete>;
	using FImplicitObject::GetTypeName;
	
	using ObjectTypeDeprecated = std::conditional_t<bInstanced, TSerializablePtr<TConcrete>, TUniquePtr<TConcrete>>;

	UE_DEPRECATED(5.4, "Constructor no longer used anymore")
	TImplicitObjectScaled(ObjectTypeDeprecated Object, const TSharedPtr<TConcrete, ESPMode::ThreadSafe>& SharedPtrForRefCount, const FVec3& Scale, FReal InMargin = 0)
	    : FImplicitObjectScaled(EImplicitObject::HasBoundingBox, Object->GetType())
	{
		check(false);
	}
	
	UE_DEPRECATED(5.4, "Constructor no longer used anymore")
	TImplicitObjectScaled(TSharedPtr<TConcrete, ESPMode::ThreadSafe> Object, const FVec3& Scale, FReal InMargin = 0)
    	    : FImplicitObjectScaled(EImplicitObject::HasBoundingBox, Object->GetType)
	{
		check(false);
	}

	UE_DEPRECATED(5.4, "Constructor no longer used anymore")
	TImplicitObjectScaled(ObjectTypeDeprecated Object, TUniquePtr<Chaos::FImplicitObject> &&ObjectOwner, const TSharedPtr<TConcrete, ESPMode::ThreadSafe>& SharedPtrForRefCount, const FVec3& Scale, FReal InMargin = 0)
		: FImplicitObjectScaled(EImplicitObject::HasBoundingBox, Object->GetType())
	{
		check(false);
	}
	
	TImplicitObjectScaled(ObjectType Object, const FVec3& Scale, FReal InMargin = 0)
		: FImplicitObjectScaled(EImplicitObject::HasBoundingBox, Object->GetType())
		, MObject(Object)
	{
		InitScaledImplicit(Scale, FRealSingle(InMargin));
	}
	
	TImplicitObjectScaled(TConcrete* Object, const FVec3& Scale, FReal InMargin = 0)
		: FImplicitObjectScaled(EImplicitObject::HasBoundingBox, Object->GetType())
		, MObject(Object)
	{
		// Transient if raw pointer not already stored in a ref counted one
		if(Object && (Object->GetRefCount() == 1))
		{
			Object->MakePersistent();
		}
		InitScaledImplicit(Scale, FRealSingle(InMargin));
	}

	TImplicitObjectScaled(const TImplicitObjectScaled<TConcrete, bInstanced>& Other) = delete;
	TImplicitObjectScaled(TImplicitObjectScaled<TConcrete, bInstanced>&& Other)
		: FImplicitObjectScaled(EImplicitObject::HasBoundingBox, Other.MObject->GetType() | ImplicitObjectType::IsScaled)
		, MObject(MoveTemp(Other.MObject))
	{
		ensureMsgf((IsScaled(MObject->GetType()) == false), TEXT("Scaled objects should not contain each other."));
		ensureMsgf((IsInstanced(MObject->GetType()) == false), TEXT("Scaled objects should not contain instances."));
		this->bIsConvex = MObject->IsConvex();
		this->bDoCollide = MObject->GetDoCollide();
		this->OuterMargin = Other.OuterMargin;
		this->MScale = Other.MScale;
		this->MInvScale = Other.MInvScale;
		this->OuterMargin = Other.OuterMargin;
		this->MLocalBoundingBox = Other.MLocalBoundingBox;
		SetMargin(Other.GetMarginf());
	}
	~TImplicitObjectScaled()= default;

	static constexpr EImplicitObjectType StaticType()
	{
		return TConcrete::StaticType() | ImplicitObjectType::IsScaled;
	}

	static const TImplicitObjectScaled<TConcrete>& AsScaledChecked(const FImplicitObject& Obj)
	{
		if constexpr (std::is_same_v<TConcrete, FImplicitObject>)
		{
			//can cast any scaled to ImplicitObject base
			check(IsScaled(Obj.GetType()));
		}
		else
		{
			check(StaticType() == Obj.GetType());
		}
		return static_cast<const TImplicitObjectScaled<TConcrete>&>(Obj);
	}

	static TImplicitObjectScaled<TConcrete>& AsScaledChecked(FImplicitObject& Obj)
	{
		if constexpr (std::is_same_v<TConcrete, FImplicitObject>)
		{
			//can cast any scaled to ImplicitObject base
			check(IsScaled(Obj.GetType()));
		}
		else
		{
			check(StaticType() == Obj.GetType());
		}
		return static_cast<TImplicitObjectScaled<TConcrete>&>(Obj);
	}

	static const TImplicitObjectScaled<TConcrete>* AsScaled(const FImplicitObject& Obj)
	{
		if constexpr (std::is_same_v<TConcrete, FImplicitObject>)
		{
			//can cast any scaled to ImplicitObject base
			return IsScaled(Obj.GetType()) ? static_cast<const TImplicitObjectScaled<TConcrete>*>(&Obj) : nullptr;
		}
		else
		{
			return StaticType() == Obj.GetType() ? static_cast<const TImplicitObjectScaled<TConcrete>*>(&Obj) : nullptr;
		}
	}

	static TImplicitObjectScaled<TConcrete>* AsScaled(FImplicitObject& Obj)
	{
		if constexpr (std::is_same_v<TConcrete, FImplicitObject>)
		{
			//can cast any scaled to ImplicitObject base
			return IsScaled(Obj.GetType()) ? static_cast<TImplicitObjectScaled<TConcrete>*>(&Obj) : nullptr;
		}
		else
		{
			return StaticType() == Obj.GetType() ? static_cast<TImplicitObjectScaled<TConcrete>*>(&Obj) : nullptr;
		}
	}

	virtual TSerializablePtr<FImplicitObject> GetInnerObject() const override
	{
		return MakeSerializable(MObject);
	}

	const TConcrete* GetUnscaledObject() const
	{
		return MObject.GetReference();
	}

	virtual EImplicitObjectType GetNestedType() const override
	{
		return MObject->GetNestedType();
	}

	virtual FReal GetRadius() const override
	{
		return (MObject->GetRadiusf() > 0.0f) ? Margin : 0.0f;
	}

	virtual FRealSingle GetRadiusf() const override
	{
		return (MObject->GetRadiusf() > 0.0f) ? Margin : 0.0f;
	}

	virtual FReal PhiWithNormal(const FVec3& X, FVec3& Normal) const override
	{
		return MObject->PhiWithNormalScaled(X, MScale, Normal);
	}

	virtual bool Raycast(const FVec3& StartPoint, const FVec3& Dir, const FReal Length, const FReal Thickness, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex) const override
	{
		ensure(Length > 0);
		ensure(FMath::IsNearlyEqual(Dir.SizeSquared(), (FReal)1, (FReal)UE_KINDA_SMALL_NUMBER));
		ensure(Thickness == 0 || (FMath::IsNearlyEqual(MScale[0], MScale[1]) && FMath::IsNearlyEqual(MScale[0], MScale[2])));	//non uniform turns sphere into an ellipsoid so no longer a raycast and requires a more expensive sweep

		const FVec3 UnscaledStart = MInvScale * StartPoint;
		const FVec3 UnscaledDirDenorm = MInvScale * Dir;
		const FReal LengthScale = UnscaledDirDenorm.Size();
		if (ensure(LengthScale > TNumericLimits<FReal>::Min()))
		{
			const FReal LengthScaleInv = FReal(1) / LengthScale;
			const FReal UnscaledLength = Length * LengthScale;
			const FVec3 UnscaledDir = UnscaledDirDenorm * LengthScaleInv;
		
			FVec3 UnscaledPosition;
			FVec3 UnscaledNormal;
			FReal UnscaledTime;

			if (MObject->Raycast(UnscaledStart, UnscaledDir, UnscaledLength, Thickness * MInvScale[0], UnscaledTime, UnscaledPosition, UnscaledNormal, OutFaceIndex))
			{
				//We double check that NewTime < Length because of potential precision issues. When that happens we always keep the shortest hit first
				const FReal NewTime = LengthScaleInv * UnscaledTime;
				if (NewTime == 0) // Normal/Position output may be uninitialized with TOI 0 so we use ray information for that as the ray origin is likely inside the shape
				{
					OutPosition = StartPoint;
					OutNormal = -Dir;
					OutTime = NewTime;
					return true;
				}
				else if (NewTime < Length) 
				{
					OutPosition = MScale * UnscaledPosition;
					OutNormal = (MInvScale * UnscaledNormal).GetSafeNormal(TNumericLimits<FRealSingle>::Min());
					OutTime = NewTime;
					return true;
				}
			}
		}
		
		return false;
	}

	/** This is a low level function and assumes the internal object has a SweepGeom function. Should not be called directly. See GeometryQueries.h : SweepQuery */
	template <typename QueryGeomType>
	bool LowLevelSweepGeom(const QueryGeomType& B, const TRigidTransform<T, d>& BToATM, const TVector<T, d>& LocalDir, const T Length, T& OutTime, TVector<T, d>& LocalPosition, TVector<T, d>& LocalNormal, int32& OutFaceIndex, TVector<T, d>& OutFaceNormal, T Thickness = 0, bool bComputeMTD = false) const
	{
		ensure(Length >= 0);
		ensure(FMath::IsNearlyEqual(LocalDir.SizeSquared(), (T)1, (T)UE_KINDA_SMALL_NUMBER));
		ensure(Thickness == 0 || (FMath::IsNearlyEqual(MScale[0], MScale[1]) && FMath::IsNearlyEqual(MScale[0], MScale[2])));

		const TVector<T, d> UnscaledDirDenorm = MInvScale * LocalDir;
		const T LengthScale = UnscaledDirDenorm.Size();
		if (ensure(LengthScale > TNumericLimits<T>::Min()))
		{
			const T LengthScaleInv = 1.f / LengthScale;
			const T UnscaledLength = Length * LengthScale;
			const TVector<T, d> UnscaledDir = UnscaledDirDenorm * LengthScaleInv;

			

			TRigidTransform<T, d> BToATMNoScale(BToATM.GetLocation() * FVec3(MInvScale), BToATM.GetRotation());

			constexpr bool bIncludeScaleInResults = std::is_same_v<TConcrete, FTriangleMeshImplicitObject>;
			if constexpr (!bIncludeScaleInResults)
			{
				TVector<T, d> UnscaledPosition;
				TVector<T, d> UnscaledNormal;
				T UnscaledTime;
				if (MObject->SweepGeom(B, BToATMNoScale, UnscaledDir, UnscaledLength, UnscaledTime, UnscaledPosition, UnscaledNormal, OutFaceIndex, OutFaceNormal, Thickness, bComputeMTD, MScale))
				{
					const T NewTime = LengthScaleInv * UnscaledTime;
					//We double check that NewTime < Length because of potential precision issues. When that happens we always keep the shortest hit first
					if (NewTime < Length)
					{
						OutTime = NewTime;
						LocalPosition = MScale * UnscaledPosition;
						LocalNormal = (MInvScale * UnscaledNormal).GetSafeNormal();
						return true;
					}
				}
			}
			else
			{
				TVector<T, d> OutScaledPosition;
				TVector<T, d> OutScaledNormal;
				T OutScaledTime;
				if (MObject->SweepGeom(B, BToATMNoScale, UnscaledDir, UnscaledLength, OutScaledTime, OutScaledPosition, OutScaledNormal, OutFaceIndex, OutFaceNormal, Thickness, bComputeMTD, MScale, bIncludeScaleInResults))
				{
					//We double check that OutScaledTime < Length because of potential precision issues. When that happens we always keep the shortest hit first
					if (OutScaledTime < Length)
					{
						OutTime = OutScaledTime;
						LocalPosition = OutScaledPosition;
						LocalNormal = OutScaledNormal.GetSafeNormal();
						if (LocalNormal.IsNearlyZero())
						{
							// Fallback: use negated sweep direction when GJK produces degenerate normal
							UE_LOGF(LogChaos, Warning, "LowLevelSweepGeom: OutScaledNormal produced zero normal after GetSafeNormal. OutScaledNormal: (%f, %f, %f), MScale: (%f, %f, %f)",
								OutScaledNormal[0], OutScaledNormal[1], OutScaledNormal[2], MScale[0], MScale[1], MScale[2]);
							LocalNormal = (-LocalDir).GetSafeNormal();
						}
						return true;
					}
				}
			}
		}

		return false;
	}

	/** This is a low level function and assumes the internal object has a SweepGeom function. Should not be called directly. See GeometryQueries.h : SweepQuery */
	template <typename QueryGeomType>
	bool LowLevelSweepGeomCCD(const QueryGeomType& B, const TRigidTransform<T, d>& BToATM, const TVector<T, d>& LocalDir, const T Length, const FReal IgnorePenetration, const FReal TargetPenetration, T& OutTOI, T& OutPhi, TVector<T, d>& LocalPosition, TVector<T, d>& LocalNormal, int32& OutFaceIndex, TVector<T, d>& OutFaceNormal) const
	{
		ensure(Length > 0);
		ensure(FMath::IsNearlyEqual(LocalDir.SizeSquared(), (T)1, (T)UE_KINDA_SMALL_NUMBER));

		const TVector<T, d> UnscaledDirDenorm = MInvScale * LocalDir;
		const T LengthScale = UnscaledDirDenorm.Size();
		if (ensure(LengthScale > TNumericLimits<T>::Min()))
		{
			const T LengthScaleInv = 1.f / LengthScale;
			const T UnscaledLength = Length * LengthScale;
			const TVector<T, d> UnscaledDir = UnscaledDirDenorm * LengthScaleInv;

			T TOI;
			T Phi;

			TRigidTransform<T, d> BToATMNoScale(BToATM.GetLocation() * FVec3(MInvScale), BToATM.GetRotation());
			
			constexpr bool bIncludeScaleInResults = std::is_same_v<TConcrete, FTriangleMeshImplicitObject>;
			if constexpr (!bIncludeScaleInResults)
			{
				TVector<T, d> UnscaledPosition;
				TVector<T, d> UnscaledNormal;
				if (MObject->SweepGeomCCD(B, BToATMNoScale, UnscaledDir, UnscaledLength, IgnorePenetration, TargetPenetration, TOI, Phi, UnscaledPosition, UnscaledNormal, OutFaceIndex, OutFaceNormal, MScale))
				{
					if (TOI < FReal(1))
					{
						OutTOI = TOI;
						OutPhi = Phi;
						LocalPosition = MScale * UnscaledPosition;
						LocalNormal = (MInvScale * UnscaledNormal).GetSafeNormal();
						return true;
					}
				}
			}
			else
			{
				TVector<T, d> ScaledPosition;
				TVector<T, d> ScaledNormal;
				if (MObject->SweepGeomCCD(B, BToATMNoScale, UnscaledDir, UnscaledLength, IgnorePenetration, TargetPenetration, TOI, Phi, ScaledPosition, ScaledNormal, OutFaceIndex, OutFaceNormal, MScale, bIncludeScaleInResults))
				{
					if (TOI < FReal(1))
					{
						OutTOI = TOI;
						OutPhi = Phi;
						LocalPosition = ScaledPosition;
						LocalNormal = ScaledNormal.GetSafeNormal();
						return true;
					}
				}
			}
		}

		return false;
	}

	template <typename QueryGeomType>
	bool GJKContactPoint(const QueryGeomType& A, const FRigidTransform3& AToBTM, const FReal Thickness, FVec3& Location, FVec3& Normal, FReal& Penetration, int32& FaceIndex) const
	{
		TRigidTransform<T, d> AToBTMNoScale(AToBTM.GetLocation() * FVec3(MInvScale), AToBTM.GetRotation());

		// Thickness is a culling distance which cannot be non-uniformly scaled. This gets passed into the ImplicitObject API unscaled so that
		// if can do the right thing internally if possible, but it makes the API a little confusing. (This is only exists used TriMesh and Heightfield.)
		// See FTriangleMeshImplicitObject::GJKContactPointImp
		const FReal UnscaledThickness = Thickness;

		auto ScaledA = MakeScaledHelper(A, MInvScale);
		return MObject->GJKContactPoint(ScaledA, AToBTMNoScale, UnscaledThickness, Location, Normal, Penetration, FaceIndex, MScale);
	}

	/** This is a low level function and assumes the internal object has a OverlapGeom function. Should not be called directly. See GeometryQueries.h : OverlapQuery */
	template <typename QueryGeomType>
	bool LowLevelOverlapGeom(const QueryGeomType& B, const TRigidTransform<T, d>& BToATM, T Thickness = 0, FMTDInfo* OutMTD = nullptr) const
	{
		ensure(Thickness == 0 || (FMath::IsNearlyEqual(MScale[0], MScale[1]) && FMath::IsNearlyEqual(MScale[0], MScale[2])));

		auto ScaledB = MakeScaledHelper(B, MInvScale);
		TRigidTransform<T, d> BToATMNoScale(BToATM.GetLocation() * FVec3(MInvScale), BToATM.GetRotation());
		return MObject->OverlapGeom(ScaledB, BToATMNoScale, Thickness, OutMTD, MScale);
	}

	// Get the index of the plane that most opposes the normal
	int32 GetMostOpposingPlane(const FVec3& Normal) const
	{
		return MObject->GetMostOpposingPlaneScaled(Normal, MScale);
	}

	// Get the nearest point on an edge of the specified face
	FVec3 GetClosestEdge(int32 PlaneIndex, const FVec3& Position, FVec3& OutEdgePos0, FVec3& OutEdgePos1) const
	{
		FVec3 EdgePos = MObject->GetClosestEdge(PlaneIndex, MInvScale * Position, OutEdgePos0, OutEdgePos1);
		OutEdgePos0 = OutEdgePos0 * MScale;
		OutEdgePos1 = OutEdgePos1 * MScale;
		return EdgePos * MScale;
	}

	// Get the nearest point on an edge of the specified face
	FVec3 GetClosestEdgePosition(int32 PlaneIndex, const FVec3& Position) const
	{
		return MObject->GetClosestEdgePosition(PlaneIndex, MInvScale * Position) * MScale;
	}

	bool GetClosestEdgeVertices(int32 PlaneIndex, const FVec3& Position, int32& OutVertexIndex0, int32& OutVertexIndex1) const
	{
		return MObject->GetClosestEdgeVertices(PlaneIndex, MInvScale * Position, OutVertexIndex0, OutVertexIndex1);
	}

	// Get an array of all the plane indices that belong to a vertex (up to MaxVertexPlanes).
	// Returns the number of planes found.
	int32 FindVertexPlanes(int32 VertexIndex, int32* OutVertexPlanes, int32 MaxVertexPlanes) const
	{
		return MObject->FindVertexPlanes(VertexIndex, OutVertexPlanes, MaxVertexPlanes);
	}

	// Get up to the 3  plane indices that belong to a vertex
	// Returns the number of planes found.
	int32 GetVertexPlanes3(int32 VertexIndex, int32& PlaneIndex0, int32& PlaneIndex1, int32& PlaneIndex2) const
	{
		return MObject->GetVertexPlanes3(VertexIndex, PlaneIndex0, PlaneIndex1, PlaneIndex2);
	}

	// The number of vertices that make up the corners of the specified face
	int32 NumPlaneVertices(int32 PlaneIndex) const
	{
		return MObject->NumPlaneVertices(PlaneIndex);
	}

	// Get the vertex index of one of the vertices making up the corners of the specified face
	int32 GetPlaneVertex(int32 PlaneIndex, int32 PlaneVertexIndex) const
	{
		return MObject->GetPlaneVertex(PlaneIndex, PlaneVertexIndex);
	}

	int32 GetEdgeVertex(int32 EdgeIndex, int32 EdgeVertexIndex) const
	{
		return MObject->GetEdgeVertex(EdgeIndex, EdgeVertexIndex);
	}

	int32 GetEdgePlane(int32 EdgeIndex, int32 EdgePlaneIndex) const
	{
		return MObject->GetEdgePlane(EdgeIndex, EdgePlaneIndex);
	}

	int32 NumPlanes() const
	{
		return MObject->NumPlanes();
	}

	int32 NumEdges() const
	{
		return MObject->NumEdges();
	}

	int32 NumVertices() const
	{
		return MObject->NumVertices();
	}

	// Get the plane at the specified index (e.g., indices from FindVertexPlanes)
	const TPlaneConcrete<FReal, 3> GetPlane(int32 FaceIndex) const
	{
		const TPlaneConcrete<FReal, 3> InnerPlane = MObject->GetPlane(FaceIndex);
		return TPlaneConcrete<FReal, 3>::MakeScaledUnsafe(InnerPlane, MScale, MInvScale);	// "Unsafe" means scale must have no zeros
	}

	void GetPlaneNX(const int32 FaceIndex, FVec3& OutN, FVec3& OutX) const
	{
		FVec3 InnerN, InnerX;
		MObject->GetPlaneNX(FaceIndex, InnerN, InnerX);
		TPlaneConcrete<FReal>::MakeScaledUnsafe(InnerN, InnerX, MScale, MInvScale, OutN, OutX);	// "Unsafe" means scale must have no zeros
	}

	// Get the vertex at the specified index (e.g., indices from GetPlaneVertex)
	const FVec3 GetVertex(int32 VertexIndex) const
	{
		const FVec3 InnerVertex = MObject->GetVertex(VertexIndex);
		return MScale * InnerVertex;
	}


	virtual int32 FindMostOpposingFace(const FVec3& Position, const FVec3& UnitDir, int32 HintFaceIndex, FReal SearchDist) const override
	{
		return MObject->FindMostOpposingFaceScaled(Position, UnitDir, HintFaceIndex, SearchDist, MScale);
	}

	virtual FVec3 FindGeometryOpposingNormal(const FVec3& DenormDir, int32 HintFaceIndex, const FVec3& OriginalNormal) const override
	{
		// @todo(chaos): we need a virtual FindGeometryOpposingNormal. Some types (Convex, Box) can just return the
		// normal from the face index without needing calculating the unscaled normals.
		ensure(FMath::IsNearlyEqual(OriginalNormal.SizeSquared(), FReal(1), FReal(UE_KINDA_SMALL_NUMBER)));

		// Get unscaled dir and normal
		const FVec3 LocalDenormDir = DenormDir * MScale;
		const FVec3 LocalOriginalNormalDenorm = OriginalNormal * MScale;
		const FReal NormalLengthScale = LocalOriginalNormalDenorm.Size();
		const FVec3 LocalOriginalNormal
			= CHAOS_ENSURE(NormalLengthScale > UE_SMALL_NUMBER)
			? LocalOriginalNormalDenorm / NormalLengthScale
			: FVec3(0, 0, 1);

		// Compute final normal
		const FVec3 LocalNormal = MObject->FindGeometryOpposingNormal(LocalDenormDir, HintFaceIndex, LocalOriginalNormal);
		FVec3 Normal = LocalNormal * MInvScale;
		FReal NormalLength = Normal.SafeNormalize(TNumericLimits<FReal>::Min());
		if (NormalLength == 0)
		{
			CHAOS_ENSURE(false);
			Normal = OriginalNormal;
		}

		return Normal;
	}

	virtual bool Overlap(const FVec3& Point, const FReal Thickness) const override
	{
		const FVec3 UnscaledPoint = MInvScale * Point;

		// TODO: consider alternative that handles thickness scaling properly in 3D, only works for uniform scaling right now
		const FReal UnscaleThickness = MInvScale[0] * Thickness; 

		return MObject->Overlap(UnscaledPoint, UnscaleThickness);
	}

	virtual Pair<FVec3, bool> FindClosestIntersectionImp(const FVec3& StartPoint, const FVec3& EndPoint, const FReal Thickness) const override
	{
		ensure(OuterMargin == 0);	//not supported: do we care?
		const FVec3 UnscaledStart = MInvScale * StartPoint;
		const FVec3 UnscaledEnd = MInvScale * EndPoint;
		auto ClosestIntersection = MObject->FindClosestIntersection(UnscaledStart, UnscaledEnd, Thickness);
		if (ClosestIntersection.Second)
		{
			ClosestIntersection.First = MScale * ClosestIntersection.First;
		}
		return ClosestIntersection;
	}

	virtual int32 FindClosestFaceAndVertices(const FVec3& Position, TArray<FVec3>& FaceVertices, FReal SearchDist = FReal(0.01)) const override
	{
		const FVec3 UnscaledPoint = MInvScale * Position;
		const FReal UnscaledSearchDist = SearchDist * MInvScale.Max();	//this is not quite right since it's no longer a sphere, but the whole thing is fuzzy anyway
		int32 FaceIndex =  MObject->FindClosestFaceAndVertices(UnscaledPoint, FaceVertices, UnscaledSearchDist);
		if (FaceIndex != INDEX_NONE)
		{
			for (FVec3& Vec : FaceVertices)
			{
				Vec = Vec * MScale;
			}
		}
		return FaceIndex;
	}


	FORCEINLINE_DEBUGGABLE FVec3 Support(const FVec3& Direction, const FReal Thickness, int32& VertexIndex) const
	{
		// Support_obj(dir) = pt => for all x in obj, pt \dot dir >= x \dot dir
		// We want Support_objScaled(dir) = Support_obj(dir') where dir' is some modification of dir so we can use the unscaled support function
		// If objScaled = Aobj where A is a transform, then we can say Support_objScaled(dir) = pt => for all x in obj, pt \dot dir >= Ax \dot dir
		// But this is the same as pt \dot dir >= dir^T Ax = (dir^TA) x = (A^T dir)^T x
		//So let dir' = A^T dir.
		//Since we only support scaling on the principal axes A is a diagonal (and therefore symmetric) matrix and so a simple component wise multiplication is sufficient
		const FVec3 UnthickenedPt = MObject->Support(Direction * MScale, 0.0f, VertexIndex) * MScale;
		return Thickness > 0 ? FVec3(UnthickenedPt + Direction.GetSafeNormal() * Thickness) : UnthickenedPt;
	}

	FORCEINLINE_DEBUGGABLE FVec3 SupportCore(const FVec3& Direction, const FReal InMargin, FReal* OutSupportDelta, int32& VertexIndex) const
	{
		return MObject->SupportCoreScaled(Direction, InMargin, MScale, OutSupportDelta, VertexIndex);
	}

	FORCEINLINE_DEBUGGABLE VectorRegister4Float SupportCoreSimd(const VectorRegister4Float& Direction, const FReal InMargin) const
	{
		FVec3 DirectionVec3;
		VectorStoreFloat3(Direction, &DirectionVec3);
		int32 VertexIndex = INDEX_NONE;
		FVec3 SupportVert = MObject->SupportCoreScaled(DirectionVec3, InMargin, MScale, nullptr, VertexIndex);
		return MakeVectorRegisterFloatFromDouble(MakeVectorRegister(SupportVert.X, SupportVert.Y, SupportVert.Z, 0.0));
	}

	// UE_DEPRECATED(5.6, "Use SetScale(FVec3f) in single precision instead")
	void SetScale(const FVec3& Scale)
	{
		SetScale(FVec3f(Scale));
	}

	void SetScale(const FVec3f& Scale)
	{
		constexpr FRealSingle MinMagnitude = 1e-6f;
		for (int Axis = 0; Axis < 3; ++Axis)
		{
			if (!CHAOS_ENSURE(FMath::Abs(Scale[Axis]) >= MinMagnitude))
			{
				MScale[Axis] = MinMagnitude;
			}
			else
			{
				MScale[Axis] = Scale[Axis];
			}

			MInvScale[Axis] = 1 / MScale[Axis];
		}
		SetMargin(OuterMargin + MScale[0] * MObject->GetMarginf());
		UpdateBounds();
	}

	const FReal GetVolume() const
	{
		return FMath::Abs(MScale.X * MScale.Y * MScale.Z) * MObject->GetVolume();
	}

	const FVec3 GetCenterOfMass() const
	{
		return MScale * MObject->GetCenterOfMass();
	}

	FRotation3 GetRotationOfMass() const
	{
		return MObject->GetRotationOfMass();
	}

	const FMatrix33 GetInertiaTensor(const FReal Mass) const
	{
		return Utilities::ScaleInertia(MObject->GetInertiaTensor(Mass), MScale, false);
	}

	const ObjectType Object() const { return MObject; }

	UE_DEPRECATED(5.4, "Please use Object instead")
	TSharedPtr<TConcrete, ESPMode::ThreadSafe> GetSharedObject() const { check(false); return nullptr; }

	virtual void Serialize(FChaosArchive& Ar) override
	{
		FChaosArchiveScopedMemory ScopedMemory(Ar, GetTypeName(), false);
		FImplicitObject::SerializeImp(Ar);
		Ar << MObject << MScale << MInvScale;
		TBox<FReal,d>::SerializeAsAABB(Ar, MLocalBoundingBox);

		Ar.UsingCustomVersion(FExternalPhysicsCustomObjectVersion::GUID);
		if (Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) < FExternalPhysicsCustomObjectVersion::ScaledGeometryIsConcrete)
		{
			this->Type = MObject->GetType() | ImplicitObjectType::IsScaled;	//update type so downcasts work
		}
	}

	virtual uint32 GetTypeHash() const override
	{
		return HashCombine(MObject->GetTypeHash(), UE::Math::GetTypeHash(MScale));
	}

	virtual uint16 GetMaterialIndex(uint32 HintIndex) const override
	{
		return MObject->GetMaterialIndex(HintIndex);
	}
	
	virtual Chaos::FImplicitObjectPtr CopyGeometry() const override
	{
		return Chaos::FImplicitObjectPtr(CopyHelper(this));
	}

	virtual Chaos::FImplicitObjectPtr CopyGeometryWithScale(const FVec3& Scale) const override
	{
		TImplicitObjectScaled<TConcrete, bInstanced>* Obj = CopyHelper(this);
		Obj->SetScale(Obj->GetScale() * Scale);
		return Chaos::FImplicitObjectPtr(Obj);
	}

	virtual FString ToString() const override
	{
		return FString::Printf(TEXT("Scaled %s, Scale: [%f, %f, %f], Margin: %f"), *MObject->ToString(), MScale.X, MScale.Y, MScale.Z, GetMarginf());
	}

private:
	ObjectType MObject;

	//needed for serialization
	TImplicitObjectScaled()
	: FImplicitObjectScaled(EImplicitObject::HasBoundingBox, StaticType())
	{}
	friend FImplicitObject;	//needed for serialization

	static TImplicitObjectScaled<TConcrete, true>* CopyHelper(const TImplicitObjectScaled<TConcrete, true>* Obj)
	{
		return new TImplicitObjectScaled<TConcrete, true>(Obj->MObject, Obj->MScale, Obj->OuterMargin);
	}

	static TImplicitObjectScaled<TConcrete, false>* CopyHelper(const TImplicitObjectScaled<TConcrete, false>* Obj)
	{
		Chaos::FImplicitObjectPtr DuplicatedShape = Obj->MObject->CopyGeometry();
	
		// We know the actual type of the underlying object pointer so we can cast it to the required type to make a copy of this implicit
		return new TImplicitObjectScaled<TConcrete, false>(reinterpret_cast<TRefCountPtr<TConcrete>&&>(DuplicatedShape), Obj->MScale, Obj->OuterMargin);
	}

	void InitScaledImplicit(const FVec3& Scale, FRealSingle InMargin = 0)
	{
		ensureMsgf((IsScaled(MObject->GetType()) == false), TEXT("Scaled objects should not contain each other."));
		ensureMsgf((IsInstanced(MObject->GetType()) == false), TEXT("Scaled objects should not contain instances."));
		switch (MObject->GetType())
		{
		case ImplicitObjectType::Transformed:
		case ImplicitObjectType::Union:
			check(false);	//scale is only supported for concrete types like sphere, capsule, convex, levelset, etc... Nothing that contains other objects
		default:
			break;
		}
		this->bIsConvex = MObject->IsConvex();
		this->bDoCollide = MObject->GetDoCollide();
		this->OuterMargin = InMargin;
		SetScale(Scale);
	}

	void UpdateBounds()
	{
		const FAABB3 UnscaledBounds = MObject->BoundingBox();
		const FVec3 Vector1 = UnscaledBounds.Min() * MScale;
		MLocalBoundingBox = FAABB3(Vector1, Vector1);	//need to grow it out one vector at a time in case scale is negative
		const FVec3 Vector2 = UnscaledBounds.Max() * MScale;
		MLocalBoundingBox.GrowToInclude(Vector2);
	}

	template <typename QueryGeomType>
	static auto MakeScaledHelper(const QueryGeomType& B, const TVector<T,d>& InvScale )
	{
		return TImplicitObjectScaled<QueryGeomType,true>(const_cast<QueryGeomType*>(&B), InvScale);
	}

	template <typename QueryGeomType>
	static auto MakeScaledHelper(const TImplicitObjectScaled<QueryGeomType>& B, const TVector<T,d>& InvScale)
	{
		//if scaled of scaled just collapse into one scaled
		TImplicitObjectScaled<QueryGeomType> ScaledB(B.Object(), InvScale * B.GetScale());
		return ScaledB;
	}

	UE_INTERNAL virtual FImplicitObjectPtr GetInnerObjectInternal() const override final
	{
		return FImplicitObjectPtr(MObject.GetReference());
	}
};

template <typename TConcrete>
using TImplicitObjectScaledNonSerializable = TImplicitObjectScaled<TConcrete, false>;

template <typename T, int d>
using TImplicitObjectScaledGeneric = TImplicitObjectScaled<FImplicitObject>;

template<typename TConcrete>
Chaos::FImplicitObjectPtr TImplicitObjectInstanced<TConcrete>::CopyGeometryWithScale(const FVec3& Scale) const
{
	return Chaos::FImplicitObjectPtr(new TImplicitObjectScaled<TConcrete, true>(MObject, Scale, 0.0));
}

template<>
struct TImplicitTypeInfo<FImplicitObjectInstanced>
{
	// Is InType derived from FImplicitObjectInstanced
	static bool IsBaseOf(const EImplicitObjectType InType)
	{
		return !!(InType & ImplicitObjectType::IsInstanced);
	}
};

template<>
struct TImplicitTypeInfo<FImplicitObjectScaled>
{
	// Is InType derived from FImplicitObjectScaled
	static bool IsBaseOf(const EImplicitObjectType InType)
	{
		return !!(InType & ImplicitObjectType::IsScaled);
	}
};


/**
 * @brief Remove the Instanced or Scaled wrapper from an ImplicitObject of a known inner type and extract the instance properties
 * @return the inner implicit object or null if the wrong type
*/
template<typename T>
const T* UnwrapImplicit(const FImplicitObject& Implicit, FVec3& OutScale, FReal &OutMargin)
{
	OutScale = FVec3(1);
	OutMargin = FReal(0);

	if (const TImplicitObjectScaled<T>* ScaledImplicit = Implicit.template GetObject<TImplicitObjectScaled<T>>())
	{
		OutScale = ScaledImplicit->GetScale();
		OutMargin = FReal(ScaledImplicit->GetMarginf());
		return ScaledImplicit->GetUnscaledObject();
	}
	else if (const TImplicitObjectInstanced<T>* InstancedImplicit = Implicit.template GetObject<TImplicitObjectInstanced<T>>())
	{
		OutMargin = FReal(InstancedImplicit->GetMarginf());
		return InstancedImplicit->GetInstancedObject();
	}
	else if (const T* RawImplicit = Implicit.template GetObject<T>())
	{
		OutMargin = FReal(RawImplicit->GetMarginf());
		return RawImplicit;
	}
	else
	{
		return nullptr;
	}
}
}

		

