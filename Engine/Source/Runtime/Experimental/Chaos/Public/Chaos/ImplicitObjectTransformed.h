// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Box.h"
#include "Chaos/ImplicitObject.h"
#include "Chaos/ShapeInstanceFwd.h"
#include "Chaos/Transform.h"
#include "ChaosArchive.h"
#include "Templates/EnableIf.h"
#include "Chaos/AABB.h"

namespace Chaos
{

namespace Rigids::Async
{
	class FRigidShapeInstanceAsync;
} // namespace Rigids::Async

inline void TImplicitObjectTransformSerializeHelper(FChaosArchive& Ar, FImplicitObjectPtr& Obj)
{
	Ar << Obj;
}

inline void TImplicitObjectTransformSerializeHelper(FChaosArchive& Ar, TSerializablePtr<FImplicitObject>& Obj)
{
	Ar << Obj;
}

inline void TImplicitObjectTransformSerializeHelper(FChaosArchive& Ar, const FImplicitObject* Obj)
{
	check(false);
}

/**
 * Transform the contained shape. If you pass a TUniquePtr to the constructor, ownership is transferred to the TransformedImplicit. If you pass a
 * SerializablePtr, the lifetime of the object must be handled externally (do not delete it before deleting the TransformedImplicit).
 * @template bSerializable Whether the shape can be serialized (usually true). Set to false for transient/stack-allocated objects. 
 */
template<class T, int d, bool bSerializable = true>
class TImplicitObjectTransformed final : public FImplicitObject
{
	using FStorage = TImplicitObjectPtrStorage<T, d, bSerializable>;
	using ObjectType = typename FStorage::PtrType;

public:
	using FImplicitObject::GetTypeName;
	
	UE_DEPRECATED(5.4, "Constructor no longer in use")
	TImplicitObjectTransformed(TUniquePtr<Chaos::FImplicitObject> &&ObjectOwner, const TRigidTransform<T, d>& InTransform)
		: FImplicitObject(EImplicitObject::HasBoundingBox, ImplicitObjectType::Transformed)
	{
		check(false);
	}

	/**
	 * Create a transform around an ImplicitObject and take control of its lifetime.
	 */
	TImplicitObjectTransformed(Chaos::FImplicitObjectPtr&& Object, const TRigidTransform<T, d>& InTransform)
		: FImplicitObject(EImplicitObject::HasBoundingBox, ImplicitObjectType::Transformed)
		, MObject(MoveTemp(Object))
		, MTransform(InTransform)
	{
		this->MLocalBoundingBox = MObject->BoundingBox().TransformedAABB(InTransform);
		this->bIsConvex = MObject->IsConvex();
		this->bDoCollide = MObject->GetDoCollide();
	}

	/**
	 * Create a transform around an ImplicitObject and take control of its lifetime.
	 */
	TImplicitObjectTransformed(const Chaos::FImplicitObjectPtr& Object, const TRigidTransform<T, d>& InTransform)
	    : FImplicitObject(EImplicitObject::HasBoundingBox, ImplicitObjectType::Transformed)
		, MObject(Object)
	    , MTransform(InTransform)
	{
		this->MLocalBoundingBox = MObject->BoundingBox().TransformedAABB(InTransform);
		this->bIsConvex = MObject->IsConvex();
		this->bDoCollide = MObject->GetDoCollide();
	}

	TImplicitObjectTransformed(const TImplicitObjectTransformed<T, d, bSerializable>& Other) = delete;
	TImplicitObjectTransformed(TImplicitObjectTransformed<T, d, bSerializable>&& Other)
	    : FImplicitObject(EImplicitObject::HasBoundingBox, ImplicitObjectType::Transformed)
	    , MObject(Other.MObject)
	    , MTransform(Other.MTransform)
	    , MLocalBoundingBox(MoveTemp(Other.MLocalBoundingBox))
	{
		this->bIsConvex = Other.MObject->IsConvex();
		this->bDoCollide = Other.MObject->GetDoCollide();
	}
	

	virtual EImplicitObjectType GetNestedType() const override
	{
		return MObject->GetNestedType();
	}
	
	virtual Chaos::FImplicitObjectPtr CopyGeometry() const
	{
		// shallow copying this just requires a copy of the pointer, which is just "this" which will convert into TRefCountPtr<FImplicitObject>
		// const_cast required here as the invasive ref count needs to be mutated
		return const_cast<TImplicitObjectTransformed*>(this);
	}
	
	virtual Chaos::FImplicitObjectPtr CopyGeometryWithScale(const FVec3& Scale) const override
	{
		// This is a deep copy - we can't take a shallow copy of this and apply the scale without modifying all other instances
		if(MObject)
		{
			//return MakeCopyWithScaleTransformed(MObjectOwner, MTransform, Scale);
			// since we cannot have a { Scaled -- Transformed -- Shape } ( scaled can only directly reference concrete shapes )
			// we need to scale the transform translation and set the Scaled on the shape itself like { (Adjusted)Transformed -- Scaled -- Shape }  
			FRigidTransform3 AdjustedTransform{ MTransform };
			AdjustedTransform.ScaleTranslation(Scale);

			Chaos::FImplicitObjectPtr ScaledObject(MObject->CopyGeometryWithScale(Scale));
			return MakeImplicitObjectPtr<TImplicitObjectTransformed<FReal,3>>(MoveTemp(ScaledObject), AdjustedTransform);
		}
		else
		{
			check(false);
			return nullptr;
		}
	}

	virtual Chaos::FImplicitObjectPtr DeepCopyGeometry() const
	{
		// Deep copy both the transformed wrapper, and the inner object
		if(MObject)
		{
			Chaos::FImplicitObjectPtr ObjectCopy(MObject->DeepCopyGeometry());
			return MakeImplicitObjectPtr<TImplicitObjectTransformed<T,d>>(MoveTemp(ObjectCopy), MTransform);
		}
		else
		{
			check(false);
			return nullptr;
		}
	}

	virtual Chaos::FImplicitObjectPtr DeepCopyGeometryWithScale(const FVec3& Scale) const override
	{
		if(MObject)
		{
			//return MakeCopyWithScaleTransformed(MObjectOwner, MTransform, Scale);
			// since we cannot have a { Scaled -- Transformed -- Shape } ( scaled can only directly reference concrete shapes )
			// we need to scale the transform translation and set the Scaled on the shape itself like { (Adjusted)Transformed -- Scaled -- Shape }  
			FRigidTransform3 AdjustedTransform{ MTransform };
			AdjustedTransform.ScaleTranslation(Scale);

			Chaos::FImplicitObjectPtr ScaledObject(MObject->DeepCopyGeometryWithScale(Scale));
			return MakeImplicitObjectPtr<TImplicitObjectTransformed<FReal,3>>(MoveTemp(ScaledObject), AdjustedTransform);
		}
		else
		{
			check(false);
			return nullptr;
		}
	}
	
	~TImplicitObjectTransformed() = default;

	static constexpr EImplicitObjectType StaticType()
	{
		return ImplicitObjectType::Transformed;
	}

	const FImplicitObject* GetTransformedObject() const
	{
		return MObject.GetReference();
	}

	virtual FReal GetMargin() const override
	{
		// If the inner shape is quadratic, we have no margin
		return (MObject->GetRadiusf() > 0.0f) ? 0.0f : Margin;
	}

	virtual FReal GetRadius() const override
	{
		// If the inner shape is quadratic, so are we
		return (MObject->GetRadiusf() > 0.0f) ? Margin : 0.0f;
	}

	bool GetDoCollide() const
	{
		return MObject->GetDoCollide();
	}

	virtual T PhiWithNormal(const TVector<T, d>& x, TVector<T, d>& Normal) const override
	{
		auto TransformedX = MTransform.InverseTransformPosition(x);
		auto Phi = MObject->PhiWithNormal(TransformedX, Normal);
		Normal = MTransform.TransformVector(Normal);
		return Phi;
	}

	virtual bool Raycast(const TVector<T, d>& StartPoint, const TVector<T, d>& Dir, const T Length, const T Thickness, T& OutTime, TVector<T, d>& OutPosition, TVector<T, d>& OutNormal, int32& OutFaceIndex) const override
	{
		const TVector<T, d> LocalStart = MTransform.InverseTransformPosition(StartPoint);
		const TVector<T, d> LocalDir = MTransform.InverseTransformVector(Dir);
		TVector<T, d> LocalPosition;
		TVector<T, d> LocalNormal;

		if (MObject->Raycast(LocalStart, LocalDir, Length, Thickness, OutTime, LocalPosition, LocalNormal, OutFaceIndex))
		{
			if (OutTime != 0.0f)
			{
				OutPosition = MTransform.TransformPosition(LocalPosition);
				OutNormal = MTransform.TransformVector(LocalNormal);
			}
			return true;
		}
		
		return false;
	}

	virtual int32 FindMostOpposingFace(const TVector<T, 3>& Position, const TVector<T, 3>& UnitDir, int32 HintFaceIndex, T SearchDistance) const override
	{
		const TVector<T, d> LocalPosition = MTransform.InverseTransformPositionNoScale(Position);
		const TVector<T, d> LocalDir = MTransform.InverseTransformVectorNoScale(UnitDir);
		return MObject->FindMostOpposingFace(LocalPosition, LocalDir, HintFaceIndex, SearchDistance);
	}

	virtual TVector<T, 3> FindGeometryOpposingNormal(const TVector<T, d>& DenormDir, int32 FaceIndex, const TVector<T, d>& OriginalNormal) const override
	{
		const TVector<T, d> LocalDenormDir = MTransform.InverseTransformVectorNoScale(DenormDir);
		const TVector<T, d> LocalOriginalNormal = MTransform.InverseTransformVectorNoScale(OriginalNormal);
		const TVector<T, d> LocalNormal = MObject->FindGeometryOpposingNormal(LocalDenormDir, FaceIndex, LocalOriginalNormal);
		return MTransform.TransformVectorNoScale(LocalNormal);
	}

	virtual bool Overlap(const TVector<T, d>& Point, const T Thickness) const override
	{
		const TVector<T, d> LocalPoint = MTransform.InverseTransformPosition(Point);
		return MObject->Overlap(LocalPoint, Thickness);
	}

	virtual Pair<TVector<T, d>, bool> FindClosestIntersectionImp(const TVector<T, d>& StartPoint, const TVector<T, d>& EndPoint, const T Thickness) const override
	{
		auto TransformedStart = MTransform.InverseTransformPosition(StartPoint);
		auto TransformedEnd = MTransform.InverseTransformPosition(EndPoint);
		auto ClosestIntersection = MObject->FindClosestIntersection(TransformedStart, TransformedEnd, Thickness);
		if (ClosestIntersection.Second)
		{
			ClosestIntersection.First = MTransform.TransformPosition(ClosestIntersection.First);
		}
		return ClosestIntersection;
	}

	virtual int32 FindClosestFaceAndVertices(const FVec3& Position, TArray<FVec3>& FaceVertices, FReal SearchDist = 0.01f) const override
	{
		const FVec3 LocalPoint = MTransform.InverseTransformPosition(Position);
		int32 FaceIndex = MObject->FindClosestFaceAndVertices(LocalPoint, FaceVertices, SearchDist);
		if (FaceIndex != INDEX_NONE)
		{
			for (FVec3& Vec : FaceVertices)
			{
				Vec = MTransform.TransformPosition(Vec);
			}
		}
		return FaceIndex;
	}

	const TRigidTransform<T, d>& GetTransform() const { return MTransform; }
	void SetTransform(const TRigidTransform<T, d>& InTransform)
	{
		MLocalBoundingBox = MObject->BoundingBox().TransformedAABB(InTransform);
		MTransform = InTransform;
	}

	virtual void AccumulateAllImplicitObjects(TArray<Pair<const FImplicitObject*, TRigidTransform<T, d>>>& Out, const TRigidTransform<T, d>& ParentTM) const override
	{
		const TRigidTransform<T, d> NewTM = MTransform * ParentTM;
		MObject->AccumulateAllImplicitObjects(Out, NewTM);
	}
	
	virtual void FindAllIntersectingObjects(TArray < Pair<const FImplicitObject*, TRigidTransform<T, d>>>& Out, const TAABB<T, d>& LocalBounds) const override
	{
		const TAABB<T, d> SubobjectBounds = LocalBounds.TransformedAABB(MTransform.Inverse());
		int32 NumOut = Out.Num();
		MObject->FindAllIntersectingObjects(Out, SubobjectBounds);
		if (Out.Num() > NumOut)
		{
			Out[NumOut].Second = Out[NumOut].Second * MTransform;
		}
	}

	virtual const TAABB<T, d> BoundingBox() const override { return MLocalBoundingBox; }

	// Calculate the tight-fitting world-space bounding box
	virtual FAABB3 CalculateTransformedBounds(const FRigidTransform3& InTransform) const
	{
		return MObject->CalculateTransformedBounds(FRigidTransform3::MultiplyNoScale(MTransform ,InTransform));
	}

	const FReal GetVolume() const
	{
		// TODO: More precise volume!
		return BoundingBox().GetVolume();
	}

	const FMatrix33 GetInertiaTensor(const FReal Mass) const
	{
		// TODO: More precise inertia!
		return BoundingBox().GetInertiaTensor(Mass);
	}

	const FVec3 GetCenterOfMass() const
	{
		// TODO: Actually compute this!
		return BoundingBox().GetCenterOfMass();
	}
	
	UE_DEPRECATED(5.4, "Please use GetGeometry instead")
	const ObjectType Object() const { return MObject; }
	
	virtual void Serialize(FChaosArchive& Ar) override
	{
		check(bSerializable);
		FChaosArchiveScopedMemory ScopedMemory(Ar, GetTypeName(), false);
		FImplicitObject::SerializeImp(Ar);
		TImplicitObjectTransformSerializeHelper(Ar, MObject);
		Ar << MTransform;
		TBox<T, d>::SerializeAsAABB(Ar, MLocalBoundingBox);

		// NOTE: Not serializing SharedObject which is a temp fix and only used in the runtime
	}

	virtual uint32 GetTypeHash() const override
	{
		// Combine the hash from the inner, non transformed object with our transform
		return HashCombine(MObject->GetTypeHash(), GetTypeHashHelper(MTransform));
	}

	virtual uint16 GetMaterialIndex(uint32 HintIndex) const override
	{
		return MObject->GetMaterialIndex(HintIndex);
	}
	
	const Chaos::FImplicitObjectRef GetGeometry() const
	{
		return MObject.GetReference();
	}
	
	void SetGeometry(const Chaos::FImplicitObjectPtr& ImplicitObject)
	{
		MObject = ImplicitObject;
	}

protected:
	virtual int32 CountObjectsInHierarchyImpl() const override final
	{
		// Include self
		return 1 + MObject->CountObjectsInHierarchy();
	}

	virtual int32 CountLeafObjectsInHierarchyImpl() const override final
	{
		// Do not include self
		return MObject->CountLeafObjectsInHierarchyImpl();
	}

	virtual void VisitOverlappingLeafObjectsImpl(
		const FAABB3& InLocalBounds,
		const FRigidTransform3& ObjectTransform,
		const int32 RootObjectIndex,
		int32& ObjectIndex,
		int32& LeafObjectIndex,
		const FImplicitHierarchyVisitor& VisitorFunc) const override final
	{
		// Skip self
		++ObjectIndex;

		// Visit child
		const FAABB3 LocalBounds = InLocalBounds.InverseTransformedAABB(MTransform);
		MObject->VisitOverlappingLeafObjectsImpl(LocalBounds, MTransform * ObjectTransform, RootObjectIndex, ObjectIndex, LeafObjectIndex, VisitorFunc);
	}

	virtual void VisitLeafObjectsImpl(
		const FRigidTransform3& ObjectTransform,
		const int32 RootObjectIndex,
		int32& ObjectIndex,
		int32& LeafObjectIndex,
		const FImplicitHierarchyVisitor& VisitorFunc) const override final
	{
		// Skip self
		++ObjectIndex;

		// Visit child
		MObject->VisitLeafObjectsImpl(MTransform * ObjectTransform, RootObjectIndex, ObjectIndex, LeafObjectIndex, VisitorFunc);
	}

	virtual bool VisitObjectsImpl(
		const FRigidTransform3& ObjectTransform,
		const int32 RootObjectIndex,
		int32& ObjectIndex,
		int32& LeafObjectIndex,
		const FImplicitHierarchyVisitorBool& VisitorFunc) const override final
	{
		// Visit self
		bool bContinue = VisitorFunc(this, ObjectTransform, RootObjectIndex, ObjectIndex, INDEX_NONE);
		++ObjectIndex;

		// Visit child
		if (bContinue)
		{
			bContinue = MObject->VisitObjectsImpl(MTransform * ObjectTransform, RootObjectIndex, ObjectIndex, LeafObjectIndex, VisitorFunc);
		}

		return bContinue;
	}

	virtual bool IsOverlappingBoundsImpl(const FAABB3& InLocalBounds) const override final
	{
		const FAABB3 LocalBounds = InLocalBounds.InverseTransformedAABB(MTransform);
		return MObject->IsOverlappingBoundsImpl(LocalBounds);
	}

private:
	Chaos::FImplicitObjectPtr MObject;
	TRigidTransform<T, d> MTransform;
	TAABB<T, d> MLocalBoundingBox;

	friend class FClusterUnionManager;

	//needed for serialization
	TImplicitObjectTransformed() : FImplicitObject(EImplicitObject::HasBoundingBox, ImplicitObjectType::Transformed) {}

	friend FImplicitObject;	//needed for serialization
	friend Rigids::Async::FRigidShapeInstanceAsync;
};

namespace Utilities
{
	UE_DEPRECATED(5.4, "Please use DuplicateGeometryWithTransform instead")
	inline TUniquePtr<FImplicitObject> DuplicateImplicitWithTransform(const FImplicitObject* const InObject, FTransform NewTransform)
	{
		check(false);
		return nullptr;
	}
	inline Chaos::FImplicitObjectPtr DuplicateGeometryWithTransform(const FImplicitObject* const InObject, FTransform NewTransform)
	{
		if(!InObject)
		{
			return nullptr;
		}

		const EImplicitObjectType OuterType = InObject->GetType();

		if(GetInnerType(OuterType) == ImplicitObjectType::Transformed)
		{
			// Take a deep copy here as we're modifying the transformed itself.
			// #TODO - Deep copy the transformed wrapper but shallow copy the internal shape as that isn't modified.
			// Likely need to expand the copy functions to handle deep copy wrappers but not concrete geoms
			Chaos::FImplicitObjectPtr NewTransformed = InObject->DeepCopyGeometry();
			TImplicitObjectTransformed<FReal, 3>* InnerTransformed = static_cast<TImplicitObjectTransformed<FReal, 3>*>(NewTransformed.GetReference());
			InnerTransformed->SetTransform(NewTransform);

			return MoveTemp(NewTransformed);
		}
		else
		{
			// Shallow copy the inner object and wrap it in a new transformed
			Chaos::FImplicitObjectPtr NewInnerObject = InObject->CopyGeometry();
			return MakeImplicitObjectPtr<TImplicitObjectTransformed<FReal, 3>>(MoveTemp(NewInnerObject), NewTransform);
		}
	}
}

template <typename T, int d>
using TImplicitObjectTransformedNonSerializable = TImplicitObjectTransformed<T, d, false>;

using FImplicitObjectTransformed = TImplicitObjectTransformed<FReal, 3>;

}
