// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"

namespace Chaos
{
	namespace ImplicitObjectType
	{
		enum
		{
			//Note: add entries in order to avoid serialization issues (but before IsInstanced)
			Sphere = 0, // warning: code assumes that this is an FSphere, but all TSpheres will think this is their type.
			Box,
			Plane,
			Capsule,
			Transformed,
			Union,
			LevelSet,
			Unknown,
			Convex,
			TaperedCylinder,
			Cylinder,
			TriangleMesh,
			HeightField,
			DEPRECATED_Scaled,	//needed for serialization of existing data
			Triangle,
			UnionClustered,
			TaperedCapsule,
			MLLevelSet,
			SkinnedTriangleMesh,
			ExtrudedTaperedCapsule,
			//Add entries above this line for serialization
			ConcreteObjectCount, // Used to ensure bitflags do not overlap concrete type
			IsWeightedLattice = 1 << 5,
			IsInstanced = 1 << 6,
			IsScaled = 1 << 7
		};
		static_assert(ConcreteObjectCount <= IsWeightedLattice, "Too many Chaos::ImplicitObjectType concrete types.");
	}

	using EImplicitObjectType = uint8;	//see ImplicitObjectType

	FORCEINLINE bool IsInstanced(EImplicitObjectType Type)
	{
		return (Type & ImplicitObjectType::IsInstanced) != 0;
	}

	FORCEINLINE bool IsScaled(EImplicitObjectType Type)
	{
		return (Type & ImplicitObjectType::IsScaled) != 0;
	}

	FORCEINLINE bool IsWeightedLattice(EImplicitObjectType Type)
	{
		return (Type & ImplicitObjectType::IsWeightedLattice) != 0;
	}

	FORCEINLINE EImplicitObjectType GetInnerType(EImplicitObjectType Type)
	{
		return Type & (~(ImplicitObjectType::IsWeightedLattice | ImplicitObjectType::IsScaled | ImplicitObjectType::IsInstanced));
	}

	namespace EImplicitObject
	{
		enum Flags
		{
			IsConvex = 1,
			HasBoundingBox = 1 << 1,
			DisableCollisions = 1 << 2
		};

		inline const int32 FiniteConvex = IsConvex | HasBoundingBox;
	}


	inline FName GetImplicitObjectTypeName(EImplicitObjectType Val)
	{
		static const FName Names[] =
		{
				TEXT("EImplicitObjectType:Sphere"),
				TEXT("EImplicitObjectType:Box"),
				TEXT("EImplicitObjectType:Plane"),
				TEXT("EImplicitObjectType:Capsule"),
				TEXT("EImplicitObjectType:Transformed"),
				TEXT("EImplicitObjectType:Union"),
				TEXT("EImplicitObjectType:LevelSet"),
				TEXT("EImplicitObjectType:Unknown"),
				TEXT("EImplicitObjectType:Convex"),
				TEXT("EImplicitObjectType:TaperedCylinder"),
				TEXT("EImplicitObjectType:Cylinder"),
				TEXT("EImplicitObjectType:TriangleMesh"),
				TEXT("EImplicitObjectType:HeightField"),
				TEXT("EImplicitObjectType:DEPRECATED_Scaled"),
				TEXT("EImplicitObjectType:Triangle"),
				TEXT("EImplicitObjectType:UnionClustered"),
				TEXT("EImplicitObjectType:TaperedCapsule"),
				TEXT("EImplicitObjectType:MLLevelSet"),
				TEXT("EImplicitObjectType:SkinnedTriangleMesh"),
				TEXT("EImplicitObjectType::ExtrudedTaperedCapsule")
		};

		static_assert(Chaos::ImplicitObjectType::ConcreteObjectCount == UE_ARRAY_COUNT(Names), "");

		return Names[Val];
	}

	inline const TCHAR* ImplicitObjectTypeToString(EImplicitObjectType Type)
	{
		switch(Type)
		{
		case ImplicitObjectType::Sphere:
			return TEXT("Sphere");
		case ImplicitObjectType::Box:
			return TEXT("Box");
		case ImplicitObjectType::Plane:
			return TEXT("Plane");
		case ImplicitObjectType::Capsule:
			return TEXT("Capsule");
		case ImplicitObjectType::Transformed:
			return TEXT("Transformed");
		case ImplicitObjectType::Union:
			return TEXT("Union");
		case ImplicitObjectType::LevelSet:
			return TEXT("LevelSet");
		case ImplicitObjectType::Unknown:
			return TEXT("Unknown");
		case ImplicitObjectType::Convex:
			return TEXT("Convex");
		case ImplicitObjectType::TaperedCylinder:
			return TEXT("TaperedCylinder");
		case ImplicitObjectType::Cylinder:
			return TEXT("Cylinder");
		case ImplicitObjectType::TriangleMesh:
			return TEXT("TriangleMesh");
		case ImplicitObjectType::HeightField:
			return TEXT("HeightField");
		case ImplicitObjectType::DEPRECATED_Scaled:
			return TEXT("DEPRECATED_Scaled");
		case ImplicitObjectType::Triangle:
			return TEXT("Triangle");
		case ImplicitObjectType::UnionClustered:
			return TEXT("UnionClustered");
		case ImplicitObjectType::TaperedCapsule:
			return TEXT("TaperedCapsule");
		case ImplicitObjectType::MLLevelSet:
			return TEXT("MLLevelSet");
		case ImplicitObjectType::SkinnedTriangleMesh:
			return TEXT("SkinnedTriangleMesh");
		case ImplicitObjectType::ExtrudedTaperedCapsule:
			return TEXT("ExtrudedTaperedCapsule");
		default:
			break;
		}

		return TEXT("Unknown");
	}

	inline EImplicitObjectType ImplicitObjectTypeFromString(const FString& InStr)
	{
		for(uint8 Index = 0; Index < ImplicitObjectType::ConcreteObjectCount; ++Index)
		{
			if(FCString::Strcmp(*InStr, ImplicitObjectTypeToString(Index)) == 0)
			{
				return EImplicitObjectType(Index);
			}
		}

		return ImplicitObjectType::Sphere;
	}

}
