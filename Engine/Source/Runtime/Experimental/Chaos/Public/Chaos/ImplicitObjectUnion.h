// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Array.h"
#include "Chaos/ImplicitObject.h"
#include "Chaos/ImplicitObjectTransformed.h"
#include "Chaos/ISpatialAcceleration.h"
#include "Chaos/GeometryParticles.h"

#include "ChaosArchive.h"

namespace Chaos
{

namespace Private
{
 	class FImplicitBVH;
	class FImplicitBVHObject;
}

namespace CVars
{
	extern bool bChaosUnionBVHEnabled;
}

class FImplicitObjectUnion : public FImplicitObject
{
  public:

	using FImplicitObject::GetTypeName;
	
	UE_DEPRECATED(5.4, "Constructor no longer used")
	FImplicitObjectUnion(TArray<TUniquePtr<FImplicitObject>>&& Objects);
	
	CHAOS_API FImplicitObjectUnion(TArray<Chaos::FImplicitObjectPtr>&& Objects);
	FImplicitObjectUnion(const FImplicitObjectUnion& Other) = delete;
	CHAOS_API FImplicitObjectUnion(FImplicitObjectUnion&& Other);
	CHAOS_API virtual ~FImplicitObjectUnion();

	FORCEINLINE static constexpr EImplicitObjectType StaticType()
	{
		return ImplicitObjectType::Union;
	}
	UE_DEPRECATED(5.4, "Please use Combine with an array of implicit object ptrs instead")
	void Combine(TArray<TUniquePtr<FImplicitObject>>& Objects) {check(false);}

	CHAOS_API void Combine(TArray<Chaos::FImplicitObjectPtr>& Objects);
	CHAOS_API void RemoveAt(int32 RemoveIndex);
	CHAOS_API void RemoveAtSortedIndices(const TArrayView<const int32>& InIndices);

	// The total number of root objects in the hierarchy (same as GetObjects().Num())
	int32 GetNumRootObjects() const
	{
		return MObjects.Num();
	}
	
	// The total number of leaf objects in the hierarchy
	int32 GetNumLeafObjects() const
	{
		return NumLeafObjects;
	}

	// Enable BVH suport for this Union. This should only be done for the root Union in a hierarchy
	void SetAllowBVH(const bool bInAllowBVH)
	{
		if (bInAllowBVH != Flags.bAllowBVH)
		{
			CheckNotLocked();

			Flags.bAllowBVH = bInAllowBVH;
			RebuildBVH();
		}
	}

	// Recurively locked the union. Once locked, it is an error to make a change to the Union or any children
	void SetLocked(const bool bInLocked)
	{
		Flags.bIsLocked = bInLocked;
	}

	void SetLockedRecursive(const bool bInLocked)
	{
		if (bInLocked != IsLocked())
		{
			for (FImplicitObject* Object : MObjects)
			{
				if (FImplicitObjectUnion* Union = Object->template GetObject<FImplicitObjectUnion>())
				{
					Union->SetLockedRecursive(bInLocked);
				}
			}

			SetLocked(bInLocked);
		}
	}

	bool IsLocked() const
	{
		return !!Flags.bIsLocked;
	}
	
	CHAOS_API virtual Chaos::FImplicitObjectPtr CopyGeometry() const;
	CHAOS_API virtual Chaos::FImplicitObjectPtr CopyGeometryWithScale(const FVec3& Scale) const override;
	CHAOS_API virtual Chaos::FImplicitObjectPtr DeepCopyGeometry() const;
	CHAOS_API virtual Chaos::FImplicitObjectPtr DeepCopyGeometryWithScale(const FVec3& Scale) const override;
	
	virtual FReal PhiWithNormal(const FVec3& x, FVec3& Normal) const override
	{
		const Chaos::FImplicitObjectsArray& Objects = MObjects;
		FReal Phi = TNumericLimits<FReal>::Max();
		bool NeedsNormalize = false;
		for (int32 i = 0; i < Objects.Num(); ++i)
		{
			if(!ensure(Objects[i]))
			{
				continue;
			}
			FVec3 NextNormal;
			FReal NextPhi = Objects[i]->PhiWithNormal(x, NextNormal);
			if (NextPhi < Phi)
			{
				Phi = NextPhi;
				Normal = NextNormal;
				NeedsNormalize = false;
			}
			else if (NextPhi == Phi)
			{
				Normal += NextNormal;
				NeedsNormalize = true;
			}
		}
		if(NeedsNormalize)
		{
			Normal.Normalize();
		}
		return Phi;
	}

	virtual const FAABB3 BoundingBox() const override { return MLocalBoundingBox; }

	virtual void AccumulateAllImplicitObjects(TArray<Pair<const FImplicitObject*, FRigidTransform3>>& Out, const FRigidTransform3& ParentTM) const
	{
		for (const Chaos::FImplicitObjectPtr& Object : GetObjects())
		{
			Object->AccumulateAllImplicitObjects(Out, ParentTM);
		}
	}

	CHAOS_API virtual void FindAllIntersectingObjects(TArray < Pair<const FImplicitObject*, FRigidTransform3>>& Out, const FAABB3& LocalBounds) const;
	
	virtual void CacheAllImplicitObjects() { RebuildBVH(); }

	virtual bool Raycast(const FVec3& StartPoint, const FVec3& Dir, const FReal Length, const FReal Thickness, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex) const override
	{
		FReal MinTime = 0;	//initialization not needed, but doing it to avoid warning
		bool bFound = false;

		for (const Chaos::FImplicitObjectPtr& Obj : MObjects)
		{
			FVec3 Position;
			FVec3 Normal;
			FReal Time;
			int32 FaceIdx;
			if (Obj->Raycast(StartPoint, Dir, Length, Thickness, Time, Position, Normal, FaceIdx))
			{
				if (!bFound || Time < MinTime)
				{
					MinTime = Time;
					OutTime = Time;
					OutPosition = Position;
					OutNormal = Normal;
					OutFaceIndex = FaceIdx;
					bFound = true;
				}
			}
		}

		return bFound;
	}

	virtual bool Overlap(const FVec3& Point, const FReal Thickness) const override
	{
		for (const Chaos::FImplicitObjectPtr& Obj : MObjects)
		{
			if (Obj->Overlap(Point, Thickness))
			{
				return true;
			}
		}

		return false;
	}

	CHAOS_API virtual void Serialize(FChaosArchive& Ar) override;

	virtual bool IsValidGeometry() const
	{
		bool bValid = FImplicitObject::IsValidGeometry();
		bValid = bValid && GetObjects().Num();
		return bValid;
	}

	// Return the list of objects that will be part of the union
	const TArray<Chaos::FImplicitObjectPtr>& GetObjects() const { return MObjects; }

	// Return the const list of objects that will be part of the union
	TArray<Chaos::FImplicitObjectPtr>& GetObjects() { return MObjects; }
	
	// The lambda returns TRUE if an object was found and iteration should stop.
	CHAOS_API void ForEachObject(TFunctionRef<bool(const FImplicitObject&, const FRigidTransform3&)> Lambda) const;

	virtual uint32 GetTypeHash() const override
	{
		uint32 Result = 0;

		// Union hash is just the hash of all internal objects
		for(const Chaos::FImplicitObjectPtr& InnerObj : MObjects)
		{
			Result = HashCombine(Result, InnerObj->GetTypeHash());
		}

		return Result;
	}

	const Private::FImplicitBVH* GetBVH() const
	{
		return BVH.Get();
	}

#if INTEL_ISPC
	// See PerParticlePBDCollisionConstraint.cpp
	// ISPC code has matching structs for interpreting FImplicitObjects.
	// This is used to verify that the structs stay the same.
	struct FISPCDataVerifier
	{
		static constexpr int32 OffsetOfMObjects() { return offsetof(FImplicitObjectUnion, MObjects); }
		static constexpr int32 SizeOfMObjects() { return sizeof(FImplicitObjectUnion::MObjects); }
	};
	friend FISPCDataVerifier;
#endif // #if INTEL_ISPC

protected:

	virtual Pair<FVec3, bool> FindClosestIntersectionImp(const FVec3& StartPoint, const FVec3& EndPoint, const FReal Thickness) const override
	{
		check(MObjects.Num());
		auto ClosestIntersection = MObjects[0]->FindClosestIntersection(StartPoint, EndPoint, Thickness);
		FReal Length = ClosestIntersection.Second ? (ClosestIntersection.First - StartPoint).Size() : 0;
		for (int32 i = 1; i < MObjects.Num(); ++i)
		{
			auto NextClosestIntersection = MObjects[i]->FindClosestIntersection(StartPoint, EndPoint, Thickness);
			if (!NextClosestIntersection.Second)
				continue;
			FReal NewLength = (NextClosestIntersection.First - StartPoint).Size();
			if (!ClosestIntersection.Second || NewLength < Length)
			{
				Length = NewLength;
				ClosestIntersection = NextClosestIntersection;
			}
		}
		return ClosestIntersection;
	}

	CHAOS_API virtual int32 CountObjectsInHierarchyImpl() const override final;
	CHAOS_API virtual int32 CountLeafObjectsInHierarchyImpl() const override final;

	CHAOS_API virtual void VisitOverlappingLeafObjectsImpl(
		const FAABB3& LocalBounds,
		const FRigidTransform3& ObjectTransform,
		const int32 RootObjectIndex,
		int32& ObjectIndex,
		int32& LeafObjectIndex,
		const FImplicitHierarchyVisitor& VisitorFunc) const override final;

	CHAOS_API virtual void VisitLeafObjectsImpl(
		const FRigidTransform3& ObjectTransform,
		const int32 RootObjectIndex,
		int32& ObjectIndex,
		int32& LeafObjectIndex,
		const FImplicitHierarchyVisitor& VisitorFunc) const override final;

	CHAOS_API virtual bool VisitObjectsImpl(
		const FRigidTransform3& ObjectTransform,
		const int32 RootObjectIndex,
		int32& ObjectIndex,
		int32& LeafObjectIndex,
		const FImplicitHierarchyVisitorBool& VisitorFunc) const override final;

	CHAOS_API virtual bool IsOverlappingBoundsImpl(
		const FAABB3& LocalBounds) const override final;

  protected:
	// Needed for serialization
	CHAOS_API FImplicitObjectUnion();
	friend FImplicitObject;

	CHAOS_API void SetNumLeafObjects(const int32 InNumLeafObjects);
	CHAOS_API void CreateBVH();
	CHAOS_API void DestroyBVH();
	CHAOS_API void RebuildBVH();

	CHAOS_API void LegacySerializeBVH(FChaosArchive& Ar);
	
	union FFLags
	{
	public:
		FFLags() : Bits(0) {}
		struct
		{
			// NOTE: Flags are serialized. Ordering must be retained
			uint8 bAllowBVH : 1;				// Are we allowed to use a BVH for this Union? (We generally only want a BVH in the root union)
			uint8 bHasBVH : 1;					// Do we currently have a BVH? Equivalent to checking BVH pointer, but used by serialization to know whether to load a BVH
			uint8 bIsLocked : 1;				// If the union is locked, it is an error to change children
		};
		uint8 Bits;
	};

	// Check if the BVH could be used and valid
	bool HasValidBVH() const;

	// Log an error we try to modify a union after it has been shared
	CHAOS_API void CheckNotLocked() const;

	// list of implicit objects that are part of the union
	TArray<Chaos::FImplicitObjectPtr> MObjects;
	
	FAABB3 MLocalBoundingBox;

	// BVH is only created when there are many objects.
	// @todo(chaos): consider registering particles that may need BVH updated in evolution instead
	TUniquePtr<Private::FImplicitBVH> BVH;
	int32 NumLeafObjects;
	FFLags Flags;
};

struct FLargeUnionClusteredImplicitInfo
{
	FLargeUnionClusteredImplicitInfo(const FImplicitObject* InImplicit, const FRigidTransform3& InTransform, const FBVHParticles* InBVHParticles)
		: Implicit(InImplicit)
		, Transform(InTransform)
		, BVHParticles(InBVHParticles)
	{
	}

	const FImplicitObject* Implicit;
	FRigidTransform3 Transform;
	const FBVHParticles* BVHParticles;
};

class FImplicitObjectUnionClustered: public FImplicitObjectUnion
{
public:
	CHAOS_API FImplicitObjectUnionClustered();
	CHAOS_API FImplicitObjectUnionClustered(TArray<Chaos::FImplicitObjectPtr>&& Objects, const TArray<FPBDRigidParticleHandle*>& OriginalParticleLookupHack = TArray<FPBDRigidParticleHandle*>());
	FImplicitObjectUnionClustered(const FImplicitObjectUnionClustered& Other) = delete;
	CHAOS_API FImplicitObjectUnionClustered(FImplicitObjectUnionClustered&& Other);
	virtual ~FImplicitObjectUnionClustered() = default;

	FORCEINLINE static constexpr EImplicitObjectType StaticType()
	{
		return ImplicitObjectType::UnionClustered;
	}

	UE_DEPRECATED(5.3, "Not supported")
	void FindAllIntersectingClusteredObjects(TArray<FLargeUnionClusteredImplicitInfo>& Out, const FAABB3& LocalBounds) const {}

	CHAOS_API TArray<FPBDRigidParticleHandle*> FindAllIntersectingChildren(const FAABB3& LocalBounds) const;

	// DO NOT USE!!
	// @todo(chaos): we should get rid of this. Instead we should hold the map/whatever on the GeometryParticle that currently owns the geom
	CHAOS_API const FPBDRigidParticleHandle* FindParticleForImplicitObject(const FImplicitObject* Object) const;

	// DO NOT USE!!
	// @todo(chaos): move this fucntionality to the geometry particle?
	CHAOS_API const FBVHParticles* GetChildSimplicial(const int32 ChildIndex) const;

private:
	// Temp hack for finding original particles
	TArray<FPBDRigidParticleHandle*> MOriginalParticleLookupHack;	
	TMap<const FImplicitObject*,FPBDRigidParticleHandle*> MCollisionParticleLookupHack;	//temp hack for finding collision particles
};

template<>
struct TImplicitTypeInfo<FImplicitObjectUnion>
{
	// @todo(chaos): this is a bit topsy-turvy because the base class needs to know about all derived classes. 
	// Ideally we would have a function like GetBaseType(InType) and implement TImplicitTypeInfo<FImplicitObjectUnionClustered> 
	// but then we'd need a runtime of type->basetype for all valid types (and then there's the bitmask complication)
	static bool IsBaseOf(const EImplicitObjectType InType)
	{
		return (InType == FImplicitObjectUnion::StaticType()) || TImplicitTypeInfo<FImplicitObjectUnionClustered>::IsBaseOf(InType);
	}
};

}
