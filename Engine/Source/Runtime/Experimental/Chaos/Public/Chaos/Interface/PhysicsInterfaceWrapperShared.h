// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Serialization/Archive.h"

enum class EQueryFlags : uint16
{
	None = 0,
	PreFilter = (1 << 2),
	PostFilter = (1 << 3),
	AnyHit = (1 << 4),
	SkipNarrowPhase = (1 << 5)
};

inline EQueryFlags operator| (EQueryFlags lhs, EQueryFlags rhs)
{
	return static_cast<EQueryFlags>(static_cast<uint16>(lhs) | static_cast<uint16>(rhs));
}

inline EQueryFlags operator&(EQueryFlags lhs, EQueryFlags rhs)
{
	return static_cast<EQueryFlags>(static_cast<uint16>(lhs) & static_cast<uint16>(rhs));
}

struct FQueryFlags
{
	FQueryFlags(EQueryFlags InFlags) : QueryFlags(InFlags) {}
	explicit operator bool() const { return !!static_cast<uint16>(QueryFlags); }
	FQueryFlags operator |(EQueryFlags Rhs) const
	{
		return FQueryFlags(QueryFlags | Rhs);
	}

	FQueryFlags& operator |=(EQueryFlags Rhs)
	{
		*this = *this | Rhs;
		return *this;
	}

	FQueryFlags operator &(EQueryFlags Rhs) const
	{
		return FQueryFlags(QueryFlags & Rhs);
	}

	FQueryFlags& operator &=(EQueryFlags Rhs)
	{
		*this = *this & Rhs;
		return *this;
	}

	bool operator==(const FQueryFlags& Other) const { return QueryFlags == Other.QueryFlags; }

	EQueryFlags QueryFlags;
};

inline FArchive& operator<<(FArchive& Ar, FQueryFlags& QueryFlags)
{
	return Ar << QueryFlags.QueryFlags;
}

/** Possible results from a scene query */
enum class EHitFlags : uint16
{
	None = 0,
	Position = (1 << 0),
	Normal = (1 << 1),
	Distance = (1 << 2),
	UV = (1 << 3),
	MTD = (1 << 9),
	FaceIndex = (1 << 10)
};

inline EHitFlags operator|(EHitFlags lhs, EHitFlags rhs)
{
	return static_cast<EHitFlags>(static_cast<uint16>(lhs) | static_cast<uint16>(rhs));
}

inline EHitFlags operator&(EHitFlags lhs, EHitFlags rhs)
{
	return static_cast<EHitFlags>(static_cast<uint16>(lhs) & static_cast<uint16>(rhs));
}

struct FHitFlags
{
	FHitFlags(EHitFlags InFlags = EHitFlags::None) : HitFlags(InFlags) {}
	explicit operator bool() const { return !!static_cast<uint16>(HitFlags); }
	FHitFlags operator |(EHitFlags Rhs) const
	{
		return FHitFlags(HitFlags | Rhs);
	}

	FHitFlags& operator |=(EHitFlags Rhs)
	{
		*this = *this | Rhs;
		return *this;
	}

	FHitFlags operator &(EHitFlags Rhs) const
	{
		return FHitFlags(HitFlags & Rhs);
	}

	FHitFlags& operator &=(EHitFlags Rhs)
	{
		*this = *this & Rhs;
		return *this;
	}

	bool operator==(const FHitFlags& Other) const { return HitFlags == Other.HitFlags; }

	EHitFlags HitFlags;
};

inline FArchive& operator<<(FArchive& Ar, FHitFlags& HitFlags)
{
	return Ar << HitFlags.HitFlags;
}
