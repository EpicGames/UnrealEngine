// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Chaos/Declares.h"
#include "Chaos/Interface/PhysicsInterfaceWrapperShared.h"
#include "Chaos/ParticleHandleFwd.h"
#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "CoreMinimal.h"
#include "Framework/ThreadContextEnum.h"
#include "HAL/PlatformCrt.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"

#include <type_traits>

namespace Chaos
{
	class FChaosArchive;
	class FPerShapeData;
}

namespace ChaosInterface
{
	struct FActorShape
	{
		Chaos::FGeometryParticle* Actor;
		const Chaos::FPerShapeData* Shape;

		CHAOS_API void Serialize(Chaos::FChaosArchive& Ar);
	};

	inline Chaos::FChaosArchive& operator<<(Chaos::FChaosArchive& Ar, FActorShape& ActorShape)
	{
		ActorShape.Serialize(Ar);
		return Ar;
	}

	struct FQueryHit : public FActorShape
	{
		FQueryHit() : FaceIndex(-1), ElementIndex(-1) {}

		/**
		Face index of touched triangle, for triangle meshes, convex meshes and height fields. Defaults to -1 if face index is not available
		*/

		int32 FaceIndex; // Signed int to match TArray's size type, and so INDEX_NONE/-1 doesn't underflow.
		FVector FaceNormal{ EForceInit::ForceInitToZero };

		int32 ElementIndex; // Currently used to indicate which shape was hit for a particle with multiple shapes.

		CHAOS_API void Serialize(Chaos::FChaosArchive& Ar);
	};

	inline Chaos::FChaosArchive& operator<<(Chaos::FChaosArchive& Ar, FQueryHit& QueryHit)
	{
		QueryHit.Serialize(Ar);
		return Ar;
	}

	struct FLocationHit : public FQueryHit
	{
		FHitFlags Flags;
		FVector WorldPosition;
		FVector WorldNormal;
		float Distance;	// LWC_TODO: Should be FVector::FReal, but that causes precision issues resulting in collision failures. Investigate!

		CHAOS_API void Serialize(Chaos::FChaosArchive& Ar);

		bool operator<(const FLocationHit& Other) const { return Distance < Other.Distance; }
	};

	inline Chaos::FChaosArchive& operator<<(Chaos::FChaosArchive& Ar, FLocationHit& LocationHit)
	{
		LocationHit.Serialize(Ar);
		return Ar;
	}

	struct FRaycastHit : public FLocationHit
	{
		float U;
		float V;

		CHAOS_API void Serialize(Chaos::FChaosArchive& Ar);

	};

	inline Chaos::FChaosArchive& operator<<(Chaos::FChaosArchive& Ar, FRaycastHit& RaycastHit)
	{
		RaycastHit.Serialize(Ar);
		return Ar;
	}

	struct FOverlapHit : public FQueryHit
	{
	};

	inline Chaos::FChaosArchive& operator<<(Chaos::FChaosArchive& Ar, FOverlapHit& OverlapHit)
	{
		OverlapHit.Serialize(Ar);
		return Ar;
	}

	struct FSweepHit : public FLocationHit
	{
	};

	inline Chaos::FChaosArchive& operator<<(Chaos::FChaosArchive& Ar, FSweepHit& SweepHit)
	{
		SweepHit.Serialize(Ar);
		return Ar;
	}

	struct FPTActorShape
	{
		Chaos::FGeometryParticleHandle* Actor;
		const Chaos::FPerShapeData* Shape;
	};

	struct FPTQueryHit : public FPTActorShape
	{
		/**
		Face index of touched triangle, for triangle meshes, convex meshes and height fields. Defaults to -1 if face index is not available
		*/

		int32 FaceIndex = -1; // Signed int to match TArray's size type, and so INDEX_NONE/-1 doesn't underflow.
		FVector FaceNormal{EForceInit::ForceInitToZero};

		int32 ElementIndex = -1; // Currently used to indicate which shape was hit for a particle with multiple shapes.
	};

	struct FPTLocationHit : public FPTQueryHit
	{
		FHitFlags Flags;
		FVector WorldPosition;
		FVector WorldNormal;
		float Distance;	// LWC_TODO: Should be FVector::FReal, but that causes precision issues resulting in collision failures. Investigate!

		bool operator<(const FPTLocationHit& Other) const { return Distance < Other.Distance; }
	};

	struct FPTRaycastHit : public FPTLocationHit
	{
		float U;
		float V;
	};

	struct FPTOverlapHit : public FPTQueryHit
	{
	};

	struct FPTSweepHit : public FPTLocationHit
	{
	};

	template<Chaos::EThreadContext Id>
	using TThreadQueryHit = std::conditional_t<Id == Chaos::EThreadContext::External, FQueryHit, FPTQueryHit>;

	template<Chaos::EThreadContext Id>
	using TThreadRaycastHit = std::conditional_t<Id == Chaos::EThreadContext::External, FRaycastHit, FPTRaycastHit>;

	template<Chaos::EThreadContext Id>
	using TThreadOverlapHit = std::conditional_t<Id == Chaos::EThreadContext::External, FOverlapHit, FPTOverlapHit>;

	template<Chaos::EThreadContext Id>
	using TThreadSweepHit = std::conditional_t<Id == Chaos::EThreadContext::External, FSweepHit, FPTSweepHit>;

#ifndef CHAOS_HIT_BUFFER_SIZE
#define CHAOS_HIT_BUFFER_SIZE 128 // Preallocated hit buffer size for traces and sweeps.
#endif // CHAOS_HIT_BUFFER_SIZE
	static_assert(CHAOS_HIT_BUFFER_SIZE > 0, "Invalid Chaos hit buffer size.");


	inline void FinishQueryHelper(TArray<FOverlapHit, TInlineAllocator<CHAOS_HIT_BUFFER_SIZE>>& Hits, const FOverlapHit& BlockingHit, bool bHasBlockingHit)
	{
		if (bHasBlockingHit)
		{
			Hits.Add(BlockingHit);
		}
	}

	inline void FinishQueryHelper(TArray<FPTOverlapHit, TInlineAllocator<CHAOS_HIT_BUFFER_SIZE>>& Hits, const FPTOverlapHit& BlockingHit, bool bHasBlockingHit)
	{
		if (bHasBlockingHit)
		{
			Hits.Add(BlockingHit);
		}
	}

	template <typename HitType>
	void FinishQueryHelper(TArray<HitType, TInlineAllocator<CHAOS_HIT_BUFFER_SIZE>>& Hits, const HitType& BlockingHit, bool bHasBlockingHit)
	{
		Hits.Sort();
		if (bHasBlockingHit)
		{
			int32 FinalNum = Hits.Num() + 1;
			for (int32 HitIdx = Hits.Num() - 1; HitIdx >= 0; --HitIdx)
			{
				if (Hits[HitIdx].Distance >= BlockingHit.Distance)
				{
					--FinalNum;
				}
				else
				{
					break;
				}
			}
			Hits.SetNum(FinalNum);
			Hits[FinalNum - 1] = BlockingHit;
		}
	}

	/** Stores the results of scene queries. This can be passed around to multiple SQAccelerators and is responsible for sorting the results and pruning based on blocking.
	IncFlushCount / DecFlushCount is used to ensure any final sort / pruning operation is done when all SQAccelerators are finished. If you are passing this into multiple accelerators
	you should call IncFlushCount / DecFlushCount yourself as otherwise each accelerator will trigger its own sort / prune.
	*/
	template<typename HitType>
	class FSQHitBuffer
	{
	public:
		FSQHitBuffer(bool bSingle = false)
			: AcceleratorDepth(0)
			, bHasBlockingHit(false)
			, bSingleResult(bSingle)
		{
			// The Hits array is preallocated through its InlineAllocator, no need to reserve.
		}

		virtual ~FSQHitBuffer() = default;

		//Called 
		void IncFlushCount()
		{
			++AcceleratorDepth;
		}

		void DecFlushCount()
		{
			--AcceleratorDepth;
			if (AcceleratorDepth == 0)
			{
				FinishQuery();
			}
		}

		bool HasHit() const { return GetNumHits(); }
		int32 GetNumHits() const { return Hits.Num(); }
		HitType* GetHits() { return Hits.GetData(); }
		const HitType* GetHits() const { return Hits.GetData(); }

		HitType* GetBlock() { return HasBlockingHit() ? &Hits.Last() : nullptr; }
		const HitType* GetBlock() const { return HasBlockingHit() ? &Hits.Last() : nullptr; }

		bool HasBlockingHit() const { return bHasBlockingHit; }

		void SetBlockingHit(const HitType& InBlockingHit)
		{
			CurrentBlockingHit = InBlockingHit;
			bHasBlockingHit = true;
		}

		void AddTouchingHit(const HitType& InTouchingHit)
		{
			Hits.Add(InTouchingHit);
		}

		const HitType* GetCurrentHit() const { return bHasBlockingHit ? &CurrentBlockingHit : nullptr; }

		/** Does not do any distance verification. This is up to the SQ code to manage */
		void InsertHit(const HitType& Hit, bool bBlocking)
		{
			// Useful place to break when debugging, but breakpoints can't be set here, as
			// this gets inlined even in a debug build.
			//__debugbreak(); 
			if (bBlocking)
			{
				SetBlockingHit(Hit);
			}
			else
			{
				AddTouchingHit(Hit);
			}
		}

		bool WantsSingleResult() const { return bSingleResult; }

	private:

		void FinishQuery()
		{
			FinishQueryHelper(Hits, CurrentBlockingHit, bHasBlockingHit);
		}

		HitType CurrentBlockingHit;
		int32 AcceleratorDepth;
		bool bHasBlockingHit;
		bool bSingleResult;

		TArray<HitType, TInlineAllocator<CHAOS_HIT_BUFFER_SIZE>> Hits;
	};

	template<typename HitType>
	class FSQSingleHitBuffer : public FSQHitBuffer<HitType>
	{
	public:
		FSQSingleHitBuffer()
			: FSQHitBuffer<HitType>(/*bSingle=*/ true)
		{}
	};

}
