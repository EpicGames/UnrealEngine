// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"

#include "Chaos/Evolution/ConstraintGroupSolver.h"
#include "Chaos/Island/IslandManagerFwd.h"

namespace Chaos
{
	namespace Private
	{
		/**
		* A set of constraints that will ne solved in sequence on a single thread. This will usually be the constraints
		* from several islands, but may be a sub set of constraints of a single island when coloring is enabled for
		* that island.
		*/
		class FPBDIslandConstraintGroupSolver : public FPBDConstraintGroupSolver
		{
		public:
			UE_NONCOPYABLE(FPBDIslandConstraintGroupSolver);

			/**
			* Init the island group 
			*/
			CHAOS_API FPBDIslandConstraintGroupSolver(FPBDIslandManager& InIslandManager);

			/**
			* Add island to the group
			* @param Island Island to be added
			*/
			CHAOS_API void AddIsland(FPBDIsland* Island);

			/**
			* Return the islands within the group
			*/
			FORCEINLINE const TArray<FPBDIsland*>& GetIslands() { return Islands; }

			/**
			 * The number of particles (dynamics, kinematic and static) in the island.
			*/
			inline int32 GetNumParticles() const { return NumParticles; }

			/**
			 * The number of constraints of all types in the island.
			*/
			inline int32 GetNumConstraints() const { return NumConstraints; }

			/**
			 * The number of constraints of the specific container type in the island.
			*/
			inline int32 GetNumConstraints(const int32 ContainerId) const { return NumContainerConstraints[ContainerId]; }

			/**
			 * Update the iteration settings based on the islands in the group, using at least as many as specified in InIterations
			*/
			CHAOS_API virtual void SetIterationSettings(const FIterationSettings& InIterations) override;

		protected:
			// Base class overrides
			CHAOS_API virtual void SetConstraintSolverImpl(const int32 ContainerId) override final;
			CHAOS_API virtual void ResetImpl() override final;
			CHAOS_API virtual void AddConstraintsImpl() override final;
			CHAOS_API virtual void GatherBodiesImpl(const FReal Dt, const int32 BeginBodyIndex, const int32 EndBodyIndex) override final;

		private:
			CHAOS_API void ApplyIterationSettingsOverrides();

			FPBDIslandManager& IslandManager;

			// Item counters used to initialize the solver containers
			static constexpr int32 NumExpectedConstraintTypes = 5;
			int32 NumParticles;
			int32 NumConstraints;
			TArray<int32, TInlineAllocator<NumExpectedConstraintTypes>> NumContainerConstraints;	// Per ContainerId

			TArray<FPBDIsland*> Islands;
		};

	}	// namespace Private


	using FPBDIslandConstraintGroupSolver UE_DEPRECATED(5.2, "Internal class moved to Private namespace") = Private::FPBDIslandConstraintGroupSolver;

}	// namespace Chaos
