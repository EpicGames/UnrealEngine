// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Island/IslandGroup.h"

namespace Chaos
{
	class FPBDConstraintContainer;

#if CSV_PROFILER_STATS
	#define CSV_CUSTOM_STAT_ISLANDGROUP_HELPER(Stat) CSV_CUSTOM_STAT(PhysicsVerbose, Stat, Stats[Stat] * 1000.0, ECsvCustomStatOp::Set);
	#define CSV_SCOPED_ISLANDGROUP_TIMING_STAT(Stat, ThreadIndex) FScopedDurationTimer Timer_##Stat(GetThreadStatAccumulator(ThreadIndex, FIslandGroupStats::Stat))

	/**
	* Accumulate various stats per thread for later aggregation into a global cost.
	* Using aligned to avoid false sharing during perf capture
	*/
	struct alignas(64) FIslandGroupStats
	{
		enum EPerIslandStat
		{
			/** Any stat added to this enum must be added to ReportStats. Can use inl if it's a problem, but seems fine to just copy paste here*/
			PerIslandSolve_GatherTotalSerialized,
			PerIslandSolve_ApplyTotalSerialized,
			PerIslandSolve_ApplyPushOutTotalSerialized,
			PerIslandSolve_ApplyProjectionTotalSerialized,
			PerIslandSolve_ScatterTotalSerialized,
			NumStats
		};

		void ReportStats() const
		{
			CSV_CUSTOM_STAT_ISLANDGROUP_HELPER(PerIslandSolve_GatherTotalSerialized);
			CSV_CUSTOM_STAT_ISLANDGROUP_HELPER(PerIslandSolve_ApplyTotalSerialized);
			CSV_CUSTOM_STAT_ISLANDGROUP_HELPER(PerIslandSolve_ApplyPushOutTotalSerialized);
			CSV_CUSTOM_STAT_ISLANDGROUP_HELPER(PerIslandSolve_ApplyProjectionTotalSerialized);
			CSV_CUSTOM_STAT_ISLANDGROUP_HELPER(PerIslandSolve_ScatterTotalSerialized);
		}

		FIslandGroupStats()
		{
			for (double& Stat : Stats) { Stat = 0.0; }
		}

		static FIslandGroupStats Flatten(const TArray<FIslandGroupStats>& PerGroupStats)
		{
			FIslandGroupStats Flat;
			for (const FIslandGroupStats& GroupStats : PerGroupStats)
			{
				for (int32 Stat = 0; Stat < NumStats; ++Stat)
				{
					Flat.Stats[Stat] += GroupStats.Stats[Stat];
				}
			}

			return Flat;
		}

		double Stats[EPerIslandStat::NumStats];
	};

#else // !CSV_PROFILER_STATS
	
	#define CSV_SCOPED_ISLANDGROUP_TIMING_STAT(Stat, ThreadIndex)

#endif

	namespace Private
	{

		/**
		 * Assigns Islands to IslandGroups, attempting to have a roughly equal number of constraints per IslandGroup.
		 * Each Island Group may be solved in parallel with the others and contains its own set of ConstraintContainerSolvers
		 * (one for each type of constraint) so that solver data access may be cache efficient during the solver phases.
		*/
		class FPBDIslandGroupManager
		{
		public:
			UE_NONCOPYABLE(FPBDIslandGroupManager)

			CHAOS_API FPBDIslandGroupManager(FPBDIslandManager& InIslandManager);
			CHAOS_API ~FPBDIslandGroupManager();

			/**
			 * Register a constraint type with the manager. This will create a ConstraintContainerSolver for the ConstraintsContainer.
			 * Constraints are solved in Priority order with lower priorities first, so higher priorities "win".
			*/
			CHAOS_API void AddConstraintContainer(FPBDConstraintContainer& ConstraintContainer, const int32 Priority = 0);

			/**
			 * Remove a previously-added container. Generall this is only needed for debuggin as cleanup is automatic on destruction.
			 */
			CHAOS_API void RemoveConstraintContainer(FPBDConstraintContainer& ConstraintContainer);

			/**
			 * Set the priority for the specified container (that must have been pre-registered with AddConstraintContainer).
			*/
			CHAOS_API void SetConstraintContainerPriority(const int32 ContainerId, const int32 Priority);

			/**
			 * The number of groups with at least one constraint in them
			*/
			inline int32 GetNumActiveGroups() const { return NumActiveGroups; }

			/**
			 * Get the specified group
			*/
			inline FPBDIslandConstraintGroupSolver* GetGroup(const int32 GroupIndex) { return IslandGroups[GroupIndex].Get(); }

			/**
			 * Returns the max number of serial constraint batches for the given ContainerId across all active island groups.
			 * Returns 0 if partitioning is disabled or no group has batches for the given container.
			*/
			UE_INTERNAL int32 GetMaxSerialConstraintBatches(const int32 ContainerId) const;

			/**
			 * Pull all the active islands from the IslandManager and assign to groups.
			 * @return 
			*/
			CHAOS_API int32 BuildGroups(const bool bIsResimming);

			/**
			 * Set the default number of iterations to perform in the Solve step (can be increased on a per-island basis by any dynamic body).
			*/
			CHAOS_API void SetIterationSettings(const FIterationSettings& InIterations);
			void SetNumPositionIterations(const int32 InNumIterations) { Iterations.SetNumPositionIterations(InNumIterations); }
			void SetNumVelocityIterations(const int32 InNumIterations) { Iterations.SetNumVelocityIterations(InNumIterations); }
			void SetNumProjectionIterations(const int32 InNumIterations) { Iterations.SetNumProjectionIterations(InNumIterations); }

			FIterationSettings GetIslandGroupIterations(const int32 GroupIndex) const { return IslandGroups[GroupIndex]->GetIterationSettings(); }

			/**
			 * Get the default iteration settings. These are the minimum number of iterations that will be run. An island may run more
			 * depending on the settings on the dynamic bodies in that island.
			*/
			const FIterationSettings& GetIterationSettings() const { return Iterations; }

			/**
			 * Solve all constraints.
			*/
			CHAOS_API void Solve(const FReal Dt);


		private:
			// Used to specify a range of bodies or constraints in an island group for more even parallelization of Gather and Scatter
			struct FIslandGroupRange
			{
				FIslandGroupRange(FPBDIslandConstraintGroupSolver* InIslandGroup, const int32 InBeginIndex, const int32 InEndIndex)
					: BeginIndex(InBeginIndex)
					, EndIndex(InEndIndex)
					, IslandGroup(InIslandGroup)
				{
				}

				// Ranges are sorted by range size
				friend bool operator<(const FIslandGroupRange& L, const FIslandGroupRange& R)
				{
					return (L.EndIndex - L.BeginIndex) > (R.EndIndex - R.BeginIndex);
				}

				int32 BeginIndex;
				int32 EndIndex;
				FPBDIslandConstraintGroupSolver* IslandGroup;
			};

			CHAOS_API void SolveSerial(const FReal Dt);
			CHAOS_API void SolveParallelFor(const FReal Dt);
			CHAOS_API void SolveParallelTasks(const FReal Dt);

			CHAOS_API void BuildGatherBatches(TArray<FIslandGroupRange>& BodyRanges, TArray<FIslandGroupRange>& ConstraintRanges);

			CHAOS_API void SolveGroupConstraintsSerial(const int32 GroupIndex, const FReal Dt);
			CHAOS_API void SolveGroupConstraintsParallelCalledFromTask(const int32 GroupIndex, const FReal Dt);
			CHAOS_API void SolveGroupConstraintsParallelNotCalledFromTask(const int32 GroupIndex, const FReal Dt);

			CHAOS_API void SolveGroupConstraintsCalledFromTask(const int32 GroupIndex, const FReal Dt);
			CHAOS_API void SolveGroupConstraintsNotCalledFromTask(const int32 GroupIndex, const FReal Dt);

			template<typename Lambda>
			TArray<UE::Tasks::FTask> DispatchBatchedTasks(
				Lambda BatchedTask,
				const int32 NumTasks,
				const int32 NumItemsPerTask,
				const int32 NumItems,
				const FReal Dt,
				const TArray<UE::Tasks::FTask>& Prereqs
			);

			FPBDIslandManager& IslandManager;
			TArray<TUniquePtr<FPBDIslandConstraintGroupSolver>> IslandGroups;
			int32 NumActiveGroups;
			int32 NumWorkerThreads;
			int32 TargetNumBodiesPerTask;
			int32 TargetNumConstraintsPerTask;
			FIterationSettings Iterations;

	#if CSV_PROFILER_STATS
			double& GetThreadStatAccumulator(const int32 ThreadIndex, const FIslandGroupStats::EPerIslandStat StatId)
			{
				if (!GroupStats.IsValidIndex(ThreadIndex))
				{
					GroupStats.SetNum(ThreadIndex + 1);
				}
				return GroupStats[ThreadIndex].Stats[StatId];
			}
			TArray<FIslandGroupStats> GroupStats;
	#endif
		};

	}	// namespace Private


	using FPBDIslandGroupManager UE_DEPRECATED(5.2, "Internal class moved to Private namespace") = Private::FPBDIslandGroupManager;

}	// namespace Chaos
