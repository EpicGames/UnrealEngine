// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Chaos/Core.h"
#include "Chaos/ConstraintHandle.h"
#include "Chaos/Framework/PoolBackedArray.h"
#include "Chaos/HandleArray.h"
#include "Chaos/Island/IslandManagerFwd.h"
#include "Chaos/ParticleHandleFwd.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/Evolution/IterationSettings.h"
#include "Chaos/Serializable.h"

namespace Chaos
{
	class FPerParticleGravity;
}

namespace Chaos::DebugDraw
{
	struct FChaosDebugDrawSettings;
}

namespace Chaos::Private
{
	class FPBDIslandMergeSet;

	struct FPBDParticleMomentumSq
	{
		FPBDParticleMomentumSq(const FRealSingle Linear = 0.0f, const FRealSingle Angular = 0.0f)
			: Linear(Linear)
			, Angular(Angular)
		{}
		FRealSingle Linear;
		FRealSingle Angular;
	};

	/**
	* A particle in the constraint graph.
	*/
	class FPBDIslandParticle
	{
	public:
		CHAOS_API FPBDIslandParticle();
		CHAOS_API FPBDIslandParticle(FGeometryParticleHandle* InParticle);
		CHAOS_API ~FPBDIslandParticle();

		FGeometryParticleHandle* GetParticle()
		{
			return Particle;
		}

		const FGeometryParticleHandle* GetParticle() const
		{
			return Particle;
		}

		// TPoolBackedItemAdapter for FPBDIslandManager::Nodes
		const int32 GetArrayIndex() const { return ArrayIndex; }
		void SetArrayIndex(const int32 InIndex) { ArrayIndex = InIndex; }
		CHAOS_API void Reuse(FGeometryParticleHandle* InParticle);
		CHAOS_API void Trash();

		// An integer ID for an island (for debug only). @see FPBDIsland::GetIslandId()
		CHAOS_API int32 GetIslandId() const;

		/**
		* Get the iterations settings for this group of islands
		*/
		FIterationSettings GetIterationSettings() const
		{
			return Particle->CastToRigidParticle()->IterationSettings();
		}

		/**
		 * Set the iterations settings for this group of islands
		*/
		void SetIterationSettings(const FIterationSettings& InIterations)
		{
			Particle->CastToRigidParticle()->SetIterationSettings(InIterations);
		}

	private:
		friend class FPBDIslandManager;
		friend class FPBDIsland;

		union FFlags
		{
		public:
			FFlags() : Bits(0) {}
			void Reset() { Bits = 0; }

			struct
			{
				uint32 bIsDynamic : 1;
				uint32 bIsSleeping : 1;
				uint32 bIsMoving : 1;
				uint32 bNeedsResim : 1;
				uint32 bWasUpdated : 1;
				uint32 bWasProcessed : 1;
				uint32 bWasTransformSetExplicitly : 1;
			};

		private:
			uint32 Bits;
		};

		// The particle we represent
		FGeometryParticleHandle* Particle = nullptr;

		// The island we are in (only if dynamic, always null for kinematics which may be in many islands)
		FPBDIsland* Island = nullptr;

		// All the edges connected to this node
		TArray<FPBDIslandConstraint*> Edges;

		// Used in a few places to track which nodes have been visited
		int32 VisitEpoch = INDEX_NONE;

		// Index into FPBDIslandManager::Nodes
		int32 ArrayIndex = INDEX_NONE;

		// Index into FPBDIsland::Nodes
		int32 IslandArrayIndex = INDEX_NONE;

		// All the flags
		FFlags Flags;

		// Distance to a kinematic through the graph
		int32 Level = 0;

		// Sleep thresholds
		FRealSingle SleepLinearThresholdSq = 0;
		FRealSingle SleepAngularThresholdSq = 0;
		int32 SleepCounterThreshold = 0;

		// Partial wake thresholds
		FRealSingle WakeLinearThresholdSq = 0;
		FRealSingle WakeAngularThresholdSq = 0;

		// Disable thresholds
		FRealSingle DisableLinearThresholdSq = 0;
		FRealSingle DisableAngularThresholdSq = 0;

		int32 ResimFrame = INDEX_NONE;
	};

	/**
	* A constraint in the constraint graph.
	*/
	class FPBDIslandConstraint
	{
	public:
		CHAOS_API FPBDIslandConstraint();
		CHAOS_API FPBDIslandConstraint(const int32 InContainerId, FConstraintHandle* InConstraint);
		CHAOS_API ~FPBDIslandConstraint();

		FConstraintHandle* GetConstraint() const
		{
			return Constraint;
		}

		uint64 GetSortKey() const
		{
			// NOTE: Level 0 means unasigned (used to be INDEX_NONE) so that this sort key still makes sense
			return (uint64(Level) << 32) | uint64(LevelSortKey);
		}

		// TPoolBackedItemAdapter for FPBDIslandManager::Edges
		const int32 GetArrayIndex() const { return ArrayIndex; }
		void SetArrayIndex(const int32 InIndex) { ArrayIndex = InIndex; }
		CHAOS_API void Reuse(const int32 InContainerId, FConstraintHandle* InConstraint);
		CHAOS_API void Trash();

		// An integer ID for an island (for debug only). @see FPBDIsland::GetIslandId()
		CHAOS_API int32 GetIslandId() const;

		bool IsSleeping() const { return Flags.bIsSleeping; }

	private:
		friend class FPBDIslandManager;
		friend class FPBDIsland;

		union FFlags
		{
		public:
			FFlags() : Bits(0) {}
			void Reset() { Bits = 0; }

			struct
			{
				uint32 bIsSleeping : 1;
				uint32 bIsNew : 1;
				uint32 bWasProcessed : 1;
			};
		private:
			uint32 Bits;
		};

		FConstraintHandle* Constraint = nullptr;
		FPBDIsland* Island = nullptr;
		FPBDIslandParticle* Nodes[2] = { nullptr, nullptr };

		// Used in a few places to track which nodes have been visited
		int32 VisitEpoch = INDEX_NONE;

		// Index into FPBDIslandManager::Edges
		int32 ArrayIndex = INDEX_NONE;

		// Index into FPBDIslandParticle::Edges for Node0 and Node1
		int32 NodeArrayIndices[2] = { INDEX_NONE, INDEX_NONE };

		// Indices into FPBDIsland::ContainerEdges
		int32 ContainerIndex = INDEX_NONE;
		int32 IslandArrayIndex = INDEX_NONE;

		// Distance to a kinematic through the graph
		int32 Level = 0;

		// Used for consistent ordering within a level
		uint32 LevelSortKey = 0;

		FFlags Flags;
	};

	/**
	* A set of connected constraints and particles.
	* A dynamic particle can only be in one island, and each island explicitly holds an array of
	* its dynamic particles. Kinematic particles may be in multiple islands and we do not hold
	* the set of these. To find kinematic particles used by an island you would need to visit all
	* the constraints.
	*/
	class FPBDIsland
	{
	public:
		CHAOS_API FPBDIsland();
		CHAOS_API ~FPBDIsland();

		int32 GetNumParticles() const
		{
			return Nodes.Num();
		}

		int32 GetNumConstraints() const
		{
			return NumEdges;
		}

		int32 GetNumContainerConstraints(const int32 ContainerId) const
		{
			return ContainerEdges[ContainerId].Num();
		}

		const FPBDIslandParticle* GetNode(const int32 NodeIndex) const
		{
			return Nodes[NodeIndex];
		}

		bool IsSleeping() const
		{
			return Flags.bIsSleeping;
		}

		int32 GetSleepCounter() const
		{
			return SleepCounter;
		}

		void SetSleepCounter(const int32 InSleepCounter)
		{
			SleepCounter = InSleepCounter;
		}

		bool IsSleepAllowed() const
		{
			return Flags.bIsSleepAllowed;
		}

		bool NeedsResim() const
		{
			return Flags.bNeedsResim;
		}

		void SetNeedsResim(const bool bInNeedsResim)
		{
			Flags.bNeedsResim = bInNeedsResim;
		}

		int32 GetResimFrame() const
		{
			return ResimFrame;
		}

		void SetResimFrame(const int32 InResimFrame)
		{
			ResimFrame = InResimFrame;
		}

		bool IsUsingCache() const
		{
			return Flags.bIsUsingCache;
		}

		void SetIsUsingCache(const bool bInIsUsingCache)
		{
			Flags.bIsUsingCache = bInIsUsingCache;
		}

		TArrayView<FPBDIslandParticle*> GetParticles()
		{
			return MakeArrayView(Nodes);
		}

		TArrayView<FPBDIslandConstraint*> GetConstraints(const int32 ContainerId)
		{
			return MakeArrayView(ContainerEdges[ContainerId]);
		}

		// An integer ID for an island. For debug only (e.g., to assign a color in debug draw) - has no internal use.
		// Not necessarily persistent between ticks.
		int32 GetIslandId() const
		{
			return ArrayIndex;
		}

		// TPoolBackedItemAdapter for FPBDIslandManager::Islands
		const int32 GetArrayIndex() const { return ArrayIndex; }
		void SetArrayIndex(const int32 InIndex) { ArrayIndex = InIndex; }
		CHAOS_API void Reuse();
		CHAOS_API void Trash();

		/**
			* Get the iterations settings for this group of islands
		*/
		FIterationSettings GetIterationSettings() const
		{
			return Iterations;
		}

		/**
		 * Set the iterations settings for this group of islands
		*/
		void SetIterationSettings(const FIterationSettings& InIterations)
		{
			Iterations = InIterations;
		}

		bool IsPartialSleepEnabled() const;

	private:
		friend class FPBDIslandManager;

		CHAOS_API void UpdateSyncState();

		union FFlags
		{
		public:
			FFlags() : Bits(0) {}
			void Reset() { Bits = 0; }

			struct
			{
				uint32 bItemsAdded : 1;
				uint32 bItemsRemoved : 1;
				uint32 bIsSleepAllowed : 1;
				uint32 bIsPartialSleepAllowedInAllNodes : 1;
				uint32 bHasSleepingNode : 1;
				uint32 bEnoughMotionlessNodesForPartialSleep : 1;
				uint32 bIsSleeping : 1;
				uint32 bWasSleeping : 1;
				uint32 bCheckSleep : 1;
				uint32 bNeedsResim : 1;
				uint32 bIsUsingCache : 1;
			};

		private:
			uint32 Bits;
		};

		// Nodes for dynamic particles in the Island (does not include kinematics)
		// @todo(chaos): we only need to track allocated nodes for sleep logic when particles have no constraints - fix this
		TArray<FPBDIslandParticle*> Nodes;

		// All edges in the island for each constraint type
		TConstraintTypeArray<TArray<FPBDIslandConstraint*>> ContainerEdges;

		// When edges are added, islands merge. The MergeSet tracks which islands are merging together
		FPBDIslandMergeSet* MergeSet = nullptr;

		// Index into FPBDIslandMergeSet::Islands
		int32 MergeSetIslandIndex = INDEX_NONE;

		// Index into FPBDIslandManager::Islands
		int32 ArrayIndex = INDEX_NONE;

		// Total number of edges in ContainerEdges[0..N]
		int32 NumEdges = 0;

		// When SleepCounter hits a threshold the island will sleep
		int32 SleepCounter = 0;

		// When DisableCounter hits a threshold the island particle will be disabled
		int32 DisableCounter = 0;

		// When SleepTypeSwitchCounter hits a threshold, the island sleep type (partial or full) 
		// will change if CVars::bChaosSolverPartialIslandSleep == true.
		int32 SleepTypeSwitchCounter = 0;

		int32 ResimFrame = INDEX_NONE;

		FFlags Flags;

		FIterationSettings Iterations;
	};

	/**
	* A set of islands to be merged together.
	*/
	class FPBDIslandMergeSet
	{
	public:
		CHAOS_API FPBDIslandMergeSet();
		CHAOS_API ~FPBDIslandMergeSet();

		// TPoolBackedItemAdapter for FPBDIslandManager::MergeSets
		const int32 GetArrayIndex() const { return ArrayIndex; }
		void SetArrayIndex(const int32 InIndex) { ArrayIndex = InIndex; }
		CHAOS_API void Reuse();
		CHAOS_API void Trash();

	private:
		friend class FPBDIslandManager;

		TArray<FPBDIsland*> Islands;
		int32 NumEdges = 0;

		// Index into FPBDIslandManager::MergeSets
		int32 ArrayIndex = INDEX_NONE;
	};

	/**
	*
	* Notes on the implementation:
	*	- Particles are nodes in the graph
	*	- Constraints are edges in the graph
	*	- Each island knows about the constraints and dynamic particle in it
	*	- Islands do not track their kinematic particles (kinematics may be in multiple islands)
	*
	* @todo(chaos): the sleep and disable logic feel like they don't belong in here. It requires
	* that we know about materials, and the Disabled Particle concept sits at a higher level
	* (in the Evolution, and in principle sleep does too since other system might need to know).
	* The main reason it is here is so that we can cache the thresholds and don't have to check
	* the materials for every particle every tick.
	*/
	class FPBDIslandManager
	{
	public:
		CHAOS_API FPBDIslandManager(FPBDRigidsSOAs& InParticles);
		CHAOS_API ~FPBDIslandManager();

		// The number of registered constraint containers
		CHAOS_API int32 GetNumConstraintContainers() const;

		// Particles management
		int32 GetNumParticles() const { return Nodes.Num(); }
		CHAOS_API void AddParticle(FGeometryParticleHandle* Particle);
		CHAOS_API void RemoveParticle(FGeometryParticleHandle* Particle);
		CHAOS_API int32 ReserveParticles(const int32 InNumParticles);
		CHAOS_API void UpdateParticleMaterial(FGeometryParticleHandle* Particle);
		CHAOS_API int32 GetParticleLevel(FGeometryParticleHandle* Particle) const;
		CHAOS_API void WakeParticleIslands(FGeometryParticleHandle* Particle);
		CHAOS_API void SleepParticle(FGeometryParticleHandle* Particle);
		CHAOS_API void FlagParticleUpdated(const FGeometryParticleHandle* Particle);
		CHAOS_API bool WasParticleTransformSetExplicitly(const FGeometryParticleHandle* Particle);
		CHAOS_API void FlagParticleTransformSetExplicitly(const FGeometryParticleHandle* Particle);

		// Constraint management
		int32 GetNumConstraints() const { return Edges.Num(); }
		CHAOS_API void AddConstraint(const int32 ContainerId, FConstraintHandle* Constraint, const TVec2<FGeometryParticleHandle*>& ConstrainedParticles);
		template<typename ConstraintContainerType> void AddContainerConstraints(ConstraintContainerType& ConstraintContainer);
		CHAOS_API void RemoveConstraint(FConstraintHandle* ConstraintHandle);
		CHAOS_API void RemoveParticleConstraints(FGeometryParticleHandle* Particle);
		CHAOS_API void RemoveParticleContainerConstraints(FGeometryParticleHandle* Particle, const int32 ContainerId);
		CHAOS_API void RemoveContainerConstraints(const int32 ContainerId);
		CHAOS_API void WakeConstraintIsland(FConstraintHandle* Constraint);

		// Access to Islands
		int32 GetNumIslands() const { return Islands.Num(); }
		const FPBDIsland* GetIsland(const int32 IslandIndex) const { return Islands[IslandIndex]; }
		CHAOS_API const FPBDIsland* GetParticleIsland(const FGeometryParticleHandle* Particle) const;
		CHAOS_API const FPBDIsland* GetConstraintIsland(const FConstraintHandle* Constraint) const;

		// Setup and update
		CHAOS_API void Reset();
		CHAOS_API void RemoveAllConstraints();
		CHAOS_API void AddConstraintContainer(const FPBDConstraintContainer& Container);
		CHAOS_API void RemoveConstraintContainer(const FPBDConstraintContainer& Container);
		CHAOS_API void SetMaterialContainers(const TArrayCollectionArray<TSerializablePtr<FChaosPhysicsMaterial>>* InPhysicsMaterials, const TArrayCollectionArray<TUniquePtr<FChaosPhysicsMaterial>>* InPerParticlePhysicsMaterials, const THandleArray<FChaosPhysicsMaterial>* InSimMaterials);
		CHAOS_API void SetGravityForces(const FPerParticleGravity* InGravity);
		CHAOS_API void SetDisableCounterThreshold(const int32 InDisableCounterThreshold);
		CHAOS_API void SetIsDeterministic(const bool bInIsDeterministic);
		CHAOS_API void SetAssignLevels(const bool bInAssignLevels);
		CHAOS_API void UpdateExplicitSleep();
		CHAOS_API void UpdateParticles();
		CHAOS_API void UpdateIslands();
		CHAOS_API void UpdateSleep(const FReal Dt = 0);
		CHAOS_API void UpdateDisable(TFunctionRef<void(FPBDRigidParticleHandle*)> ParticleDisableFunctor);
		CHAOS_API void EndTick();

		// Levels and Colors for sorting an parallelization
		CHAOS_API int32 GetParticleLevel(const FPBDIslandParticle* Node) const;
		CHAOS_API int32 GetParticleColor(const FPBDIslandParticle* Node) const;
		CHAOS_API int32 GetConstraintLevel(const FPBDIslandConstraint* Edge) const;
		CHAOS_API int32 GetConstraintColor(const FPBDIslandConstraint* Edge) const;

		/**
		* Visit all the awake constraints from the specified container.
		* @tparam VisitorType void(const FPBDIslandConstraint*)
		*/
		template<typename VisitorType> void VisitAwakeConstraints(const int32 ContainerId, const VisitorType& Visitor);
		template<typename VisitorType> void VisitAwakeConstConstraints(const int32 ContainerId, const VisitorType& Visitor) const;

		/**
		* Visit all the constraints from the specified container
		* @tparam VisitorType void(const FPBDIslandConstraint*)
		*/
		template<typename VisitorType> void VisitConstraints(const int32 ContainerId, const VisitorType& Visitor);
		template<typename VisitorType> void VisitConstConstraints(const int32 ContainerId, const VisitorType& Visitor) const;

		// Resim functionality
		CHAOS_API int32 GetParticleResimFrame(const FGeometryParticleHandle* Particle) const;
		CHAOS_API void SetParticleResimFrame(FGeometryParticleHandle* Particle, const int32 ResimFrame);
		CHAOS_API void ResetParticleResimFrame(const int32 ResetFrame = INDEX_NONE);

		// The following is intended for testing, debugging and visualization only and is not considered part of the API
		FPBDIsland* GetIsland(const int32 IslandIndex) { return Islands[IslandIndex]; }
		CHAOS_API int32 GetIslandIndex(const FPBDIsland* Island) const;
		CHAOS_API int32 GetIslandArrayIndex(const FPBDIslandConstraint* Edge) const;
		CHAOS_API TArray<const FPBDIsland*> FindParticleIslands(const FGeometryParticleHandle* Particle) const;
		CHAOS_API void FindParticleIslands(const FGeometryParticleHandle* Particle, TArray<const FPBDIsland*>& OutParticleIslands) const;
		CHAOS_API TArray<const FGeometryParticleHandle*> FindParticlesInIslands(const TArray<const FPBDIsland*> Islands) const;
		CHAOS_API void FindParticlesInIslands(const TArray<const FPBDIsland*>& InIslands, TArray<const FGeometryParticleHandle*>& OutIslandParticles) const;
		CHAOS_API TArray<const FConstraintHandle*> FindConstraintsInIslands(const TArray<const FPBDIsland*> Islands, int32 ContainerId) const;

		// Debug draw
#if CHAOS_DEBUG_DRAW
		CHAOS_API void DebugDrawSleepState(const DebugDraw::FChaosDebugDrawSettings* DebugDrawSettings) const;
#endif

		// Deprecated API
		UE_DEPRECATED(5.3, "Use Reset") void RemoveConstraints() { Reset(); }
		UE_DEPRECATED(5.3, "Use UpdateParticles") void InitializeGraph(const TParticleView<FPBDRigidParticles>& InParticles) { UpdateParticles(); }
		UE_DEPRECATED(5.3, "Use GetNumIslands") int32 NumIslands() const { return GetNumIslands(); }
		UE_DEPRECATED(5.3, "Use GetParticleIsland") const FPBDIsland* GetIsland(const FGeometryParticleHandle* Particle) const { return GetParticleIsland(Particle); }
		UE_DEPRECATED(5.3, "Use RemoveConstraint without ContainerId") void RemoveConstraint(const int32 ContainerId, FConstraintHandle* Constraint) { RemoveConstraint(Constraint); }

	private:
		// @todo(chaos): for access to non-const GetIsland(). Try to remove these...
		friend class FPBDIslandGroupManager;
		friend class FPBDRigidsEvolutionGBF;

		// Node management
		CHAOS_API FPBDIslandParticle* GetGraphNode(const FGeometryParticleHandle* Particle) const;
		CHAOS_API FPBDIslandParticle* GetGraphNode(const FTransientGeometryParticleHandle& Particle) const;
		CHAOS_API FPBDIslandParticle* CreateGraphNode(FGeometryParticleHandle* Particle);
		CHAOS_API FPBDIslandParticle* GetOrCreateGraphNode(FGeometryParticleHandle* Particle);
		CHAOS_API void DestroyGraphNode(FPBDIslandParticle* Node);

		// Edge management
		CHAOS_API FPBDIslandConstraint* GetGraphEdge(const FConstraintHandle* Constraint);
		CHAOS_API const FPBDIslandConstraint* GetGraphEdge(const FConstraintHandle* Constraint) const;
		CHAOS_API FPBDIslandConstraint* CreateGraphEdge(const int32 ContainerId, FConstraintHandle* Constraint, const TVec2<FGeometryParticleHandle*>& ConstrainedParticles);
		CHAOS_API void DestroyGraphEdge(FPBDIslandConstraint* Edge);
		CHAOS_API void BindEdgeToNodes(FPBDIslandConstraint* Edge, FPBDIslandParticle* Node0, FPBDIslandParticle* Node1);
		CHAOS_API void UnbindEdgeFromNodes(FPBDIslandConstraint* Edge);
		CHAOS_API void UpdateGraphNodeSleepSettings(FPBDIslandParticle* Node);

		// Per-tick state transfer to nodes/edges
		CHAOS_API void UpdateGraphNode(FPBDIslandParticle* Node);
		CHAOS_API void UpdateGraphEdge(FPBDIslandConstraint* Edge);

		// Island management
		CHAOS_API FPBDIsland* CreateIsland();
		CHAOS_API void DestroyIsland(FPBDIsland* Island);
		CHAOS_API void AssignNodeIsland(FPBDIslandParticle* Node);
		CHAOS_API void AssignEdgeIsland(FPBDIslandConstraint* Edge);
		CHAOS_API void AddNodeToIsland(FPBDIslandParticle* Node, FPBDIsland* Island);
		CHAOS_API void RemoveNodeFromIsland(FPBDIslandParticle* Node);
		CHAOS_API void DestroyIslandNodes(FPBDIsland* Island);
		CHAOS_API void AddEdgeToIsland(FPBDIslandConstraint* Edge, FPBDIsland* Island);
		CHAOS_API void RemoveEdgeFromIsland(FPBDIslandConstraint* Edge);
		CHAOS_API void WakeNodeIslands(const FPBDIslandParticle* Node);
		CHAOS_API void EnqueueIslandCheckSleep(FPBDIsland* Island, const bool bIsSleepAllowed);
		CHAOS_API void UpdateIslandPartialSleepAllowed(FPBDIsland* Island, const FGeometryParticleHandle* Particle);
		CHAOS_API void ResetIslandPartialSleepAllowed();
		CHAOS_API void UpdateIslandHasSleepingNode(FPBDIsland* Island, const FGeometryParticleHandle* Particle);

		// Island merge set management
		CHAOS_API FPBDIsland* MergeNodeIslands(FPBDIslandParticle* Node0, FPBDIslandParticle* Node1);
		CHAOS_API FPBDIsland* MergeIslands(FPBDIsland* Island0, FPBDIsland* Island1);
		CHAOS_API FPBDIslandMergeSet* CreateMergeSet(FPBDIsland* Island0, FPBDIsland* Island1);
		CHAOS_API void DestroyMergeSet(FPBDIslandMergeSet* IslandMergeSet);
		CHAOS_API void AddIslandToMergeSet(FPBDIsland* Island, FPBDIslandMergeSet* IslandMergeSet);
		CHAOS_API void RemoveIslandFromMergeSet(FPBDIsland* Island);
		CHAOS_API FPBDIslandMergeSet* CombineMergeSets(FPBDIslandMergeSet* IslandMergeSetParent, FPBDIslandMergeSet* IslandMergeSetChild);
		CHAOS_API FPBDIsland* GetMergeSetParentIsland(FPBDIslandMergeSet* MergeSet, int32& OutNumEdges, const TArrayView<int32>& OutNumContainerEdges);

		// Merge/split processing
		CHAOS_API void ProcessIslands();
		CHAOS_API void ProcessWakes();
		CHAOS_API void ProcessMerges();
		CHAOS_API void ProcessSplits();
		CHAOS_API void AssignLevels();
		CHAOS_API void ProcessIslandMerge(FPBDIsland* ParentIsland, FPBDIsland* ChildIsland);
		CHAOS_API void ProcessIslandSplits(FPBDIsland* Island);

		// Sorting
		CHAOS_API void AssignIslandLevels(FPBDIsland* Island);
		CHAOS_API void SortIslandEdges(FPBDIsland* Island);
		CHAOS_API void RandomizeIslandEdges(FPBDIsland* Island);

		// Island Sleeping
		CHAOS_API void ProcessSleep(const FRealSingle Dt);
		CHAOS_API void ProcessParticlesSleep(const FRealSingle Dt);
		CHAOS_API void ProcessIslandSleep(FPBDIsland* Island, const FRealSingle Dt);
		CHAOS_API void PropagateIslandSleep(FPBDIsland* Island);
		CHAOS_API void PropagateIslandSleepToParticles(FPBDIsland* Island);
		CHAOS_API void PropagateIslandSleepToConstraints(FPBDIsland* Island);

		// Partial Island Sleeping
		CHAOS_API void ProcessPartialIslandSleep(FPBDIsland* Island, const FRealSingle Dt);
		CHAOS_API void SleepNonMovingParticles(FPBDIsland* Island, const FRealSingle Dt);
		CHAOS_API bool RewakeConstraintsNotSupportingPartialSleep(FPBDIsland* Island);
		CHAOS_API void SleepConstraintsBetweenNonAwakeParticles(FPBDIsland* Island);
		CHAOS_API void PostStepPartialWake(FPBDIsland* Island);
		CHAOS_API void SwitchIslandSleepType(FPBDIsland* Island, const int32 MotionlessCount);
		CHAOS_API void ValidateConstraintSleepState(const FPBDIsland* Island);

		// Partial Island Waking
		CHAOS_API void ProcessPartialIslandWake(FPBDIsland* Island, const bool bProcessEdgeBasedWake);
		CHAOS_API bool PropagatePartialIslandWake(TArray<FPBDIslandParticle*>& NodeQueue, TArray<FPBDParticleMomentumSq>& MomentumQueue,
			TArray<FGeometryParticleHandle*>& ParticlesToWakeUp, const int32 NumNodesConstraintWake, const bool bTriggerNodesAlwaysPropagate);
		CHAOS_API bool ProcessTriggerParticle(FPBDIslandParticle* Node, TArray<FPBDIslandParticle*>& NodeQueue, TArray<FPBDParticleMomentumSq>& MomentumQueue, const bool bAlwaysTrigger);
		CHAOS_API bool ProcessParticleWake(FPBDIslandParticle* Node, const FPBDIslandParticle* OtherNode, const FPBDParticleMomentumSq* OtherMomentum,
			TArray<FPBDIslandParticle*>& NodeQueue, TArray<FPBDParticleMomentumSq>& MomentumQueue,
			TArray<FGeometryParticleHandle*>& WakeUpQueue, const bool bAlwaysPropagate);
		CHAOS_API void SyncConstraintSleepStateToParticles(FPBDIsland* Island);
		CHAOS_API void WakeUpParticles(TArray<FGeometryParticleHandle*>& ParticlesToWake, const bool bResetSleepCounter = true);
		CHAOS_API void ResetParticleSleepCounter(FPBDIsland* Island);
		CHAOS_API void ResetEdgeFlags(FPBDIsland* Island);
		CHAOS_API void ResetNodeFlags(FPBDIsland* Island);
		CHAOS_API void ResetEdgeFlags();
		CHAOS_API void ResetNodeFlags();

		// Disabling
		CHAOS_API void ProcessDisable(TFunctionRef<void(FPBDRigidParticleHandle*)> ParticleDisableFunctor);
		CHAOS_API void ProcessParticlesDisable(TArray<FPBDRigidParticleHandle*>& OutDisableParticles);
		CHAOS_API void ProcessIslandDisable(FPBDIsland* Island, TArray<FPBDRigidParticleHandle*>& OutDisableParticles);

		CHAOS_API void FinalizeIslands();
		CHAOS_API void ApplyDeterminism();
		CHAOS_API bool Validate() const;
		CHAOS_API void UpdateAllParticleSleepMetrics(const FRealSingle Dt);

		CHAOS_API int32 GetNextVisitEpoch();
		CHAOS_API bool ShouldAssignLevels(const FPBDIsland* Island) const;

		// Ideally the island manager would not know about the particles, materials etc, but
		// it is currently required for particle sleep management
		FPBDRigidsSOAs& Particles;
		const TArrayCollectionArray<TSerializablePtr<FChaosPhysicsMaterial>>* PhysicsMaterials;
		const TArrayCollectionArray<TUniquePtr<FChaosPhysicsMaterial>>* PerParticlePhysicsMaterials;
		const THandleArray<FChaosPhysicsMaterial>* SimMaterials;
		const FPerParticleGravity* Gravity;
		int32 DisableCounterThreshold = TNumericLimits<int32>::Max();

		// The nodes, edges and islands in the graph
		TPoolBackedArray<FPBDIslandParticle> Nodes;
		TPoolBackedArray<FPBDIslandConstraint> Edges;
		TPoolBackedArray<FPBDIsland> Islands;

		// For tracking what islands need to be merged each tick
		TPoolBackedArray<FPBDIslandMergeSet> MergeSets;

		// All the constraint types registered with the graph
		TArray<const FPBDConstraintContainer*> ConstraintContainers;

		// A monotonically increasing key used to sort edges at the same level
		int32 NextLevelSortKey = 0;

		// A counter used to check if a node or edge has been visited in a loop
		int32 NextVisitEpoch = 0;

		// Whether we require determinism
		bool bIsDeterministic = false;

		// Whether we should assign levels (for shock propagation)
		bool bAssignLevels = true;

		// Access to FPBDIslandParticle::Edges
		// Access to FPBDIslandParticle::IslandArrayIndex
		UE_MT_DECLARE_RW_ACCESS_DETECTOR(LocalNodeAccessDetector);

		// Access to FPBDIslandConstraint::Nodes[2], FPBDIslandConstraint::NodeArrayIndices[2]
		// Access to FPBDIslandConstraint::ContainerIndex, FPBDIslandConstraint::IslandArrayIndex
		UE_MT_DECLARE_RW_ACCESS_DETECTOR(LocalEdgeAccessDetector);

		// Access to FPBDIsland::Nodes, FPBDIsland::ContainerEdges
		UE_MT_DECLARE_RW_ACCESS_DETECTOR(LocalIslandAccessDetector);

		// Access to FPBDIslandManager::Nodes
		UE_MT_DECLARE_RW_ACCESS_DETECTOR(GlobalNodesAccessDetector);

		// Access to FPBDIslandManager::Edges
		UE_MT_DECLARE_RW_ACCESS_DETECTOR(GlobalEdgesAccessDetector);
	};


	template<typename ConstraintContainerType>
	void FPBDIslandManager::AddContainerConstraints(ConstraintContainerType& ConstraintContainer)
	{
		const int32 ContainerId = ConstraintContainer.GetContainerId();
		for (FConstraintHandle* ConstraintHandle : ConstraintContainer.GetConstraintHandles())
		{
			if (ConstraintHandle != nullptr)
			{
				const bool bIsInGraph = ConstraintHandle->IsInConstraintGraph();
				const bool bShouldBeInGraph = ConstraintHandle->IsEnabled();
				if (bShouldBeInGraph && !bIsInGraph)
				{
					AddConstraint(ContainerId, ConstraintHandle, ConstraintHandle->GetConstrainedParticles());
				}
				else if (!bShouldBeInGraph && bIsInGraph)
				{
					RemoveConstraint(ConstraintHandle);
				}
			}
		}
	}

	template<typename VisitorType>
	void FPBDIslandManager::VisitAwakeConstConstraints(const int32 ContainerId, const VisitorType& Visitor) const
	{
		for (FPBDIsland* Island : Islands)
		{
			if (!Island->Flags.bIsSleeping)
			{
				for (FPBDIslandConstraint* Edge : Island->ContainerEdges[ContainerId])
				{
					if (!Edge->Flags.bIsSleeping)
					{
						Visitor(Edge);
					}
				}
			}
		}
	}

	template<typename VisitorType>
	void FPBDIslandManager::VisitAwakeConstraints(const int32 ContainerId, const VisitorType& Visitor)
	{
		const_cast<FPBDIslandManager*>(this)->VisitAwakeConstConstraints(ContainerId,
			[&Visitor](const FPBDIslandConstraint* Edge)
			{
				Visitor(const_cast<FPBDIslandConstraint*>(Edge));
			});
	}

	template<typename VisitorType>
	void FPBDIslandManager::VisitConstConstraints(const int32 ContainerId, const VisitorType& Visitor) const
	{
		for (const FPBDIslandConstraint* Edge : Edges)
		{
			if (Edge->ContainerIndex == ContainerId)
			{
				Visitor(Edge);
			}
		}
	}

	template<typename VisitorType>
	void FPBDIslandManager::VisitConstraints(const int32 ContainerId, const VisitorType& Visitor)
	{
		const_cast<FPBDIslandManager*>(this)->VisitConstConstraints(ContainerId,
			[&Visitor](const FPBDIslandConstraint* Edge)
			{
				Visitor(const_cast<FPBDIslandConstraint*>(Edge));
			});
	}

} // namespace Chaos::Private
