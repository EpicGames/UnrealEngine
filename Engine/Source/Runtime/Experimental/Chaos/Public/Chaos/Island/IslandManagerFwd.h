// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/ParticleHandleFwd.h"

namespace Chaos
{
	template<class T> class TArrayCollectionArray;
	template<typename T, int D> class TPBDRigidParticles;
	template <typename T> class TParticleView;

	class FChaosPhysicsMaterial;
	class FConstraintContainerSolver;
	class FConstraintHandle;
	class FIterationSettings;
	class FPBDConstraintContainer;
	class FPBDRigidsEvolutionGBF;
	class FPBDRigidsSOAs;

	using FPBDRigidParticles = TPBDRigidParticles<FReal, 3>;
}

namespace Chaos::Private
{
	class FPBDIsland;
	class FPBDIslandConstraint;
	class FPBDIslandGroup;
	class FPBDIslandGroupManager;
	class FPBDIslandManager;
	class FPBDIslandParticle;

	// The expected number of constraint types we support. Actually this is the number of constraint containers we 
	// have, but currently we only have one container per constraint type.
	// @todo(chaos): we need to do something better when we support user constraints.
	inline constexpr int32 ConstraintGraphNumConstraintTypes = 5;

	// A typedef for arrays of items indexed by constraint type E.g., TConstraintTypeArray<int32> to hold a count of 
	// constrains per type, or TConstraintTypeArray<FConstraintHandle*> to hold a set of constraint handles by type 
	template<typename T> using TConstraintTypeArray = TArray<T, TInlineAllocator<ConstraintGraphNumConstraintTypes>>;
}


// DEPRECATED STUFF BELOW HERE
namespace Chaos
{
	// These classes are intended for internal use only and were moved to the private namespace in 5.2
	using FPBDIslandManager UE_DEPRECATED(5.2, "Internal class moved to Private namespace") = Private::FPBDIslandManager;
	using FPBDIslandParticle UE_DEPRECATED(5.2, "Internal class moved to Private namespace") = Private::FPBDIslandParticle;
	using FPBDIslandConstraint UE_DEPRECATED(5.2, "Internal class moved to Private namespace") = Private::FPBDIslandConstraint;
	using FPBDIsland UE_DEPRECATED(5.2, "Internal class moved to Private namespace") = Private::FPBDIsland;
	using FPBDConstraintGraph UE_DEPRECATED(5.2, "Internal class moved to Private namespace") = Private::FPBDIslandManager;
}
