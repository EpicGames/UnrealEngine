// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

namespace Chaos
{

	/**
	 * A graph used by the constraints system.
	 * Supports 
	 *	- partitioning vertices into connected groups (islands)
	 *	- assigning a level to each vertex based on distance from "level 0" vertices
	 *	- colorizing vertices
	 */
	class FColoringGraph
	{
	public:

		int32 GetNumIslands() const
		{
			return Islands.Num();
		}

		int32 GetVertexIsland(int32 VertexIndex) const
		{
			if (VertexIndex != INDEX_NONE)
			{
				return Vertices[VertexIndex].Island;
			}
			return INDEX_NONE;
		}

		int32 GetVertexIslandSize(int32 VertexIndex) const
		{
			if (VertexIndex != INDEX_NONE)
			{
				int32 IslandIndex = Vertices[VertexIndex].Island;
				return Islands[IslandIndex].VertexIndices.Num();
			}
			return INDEX_NONE;
		}

		int32 GetVertexLevel(int32 VertexIndex) const
		{
			if (VertexIndex != INDEX_NONE)
			{
				return Vertices[VertexIndex].Level;
			}
			return INDEX_NONE;
		}

		void SetVertexLevel(int32 VertexIndex, int32 Level)
		{
			if (VertexIndex != INDEX_NONE)
			{
				Vertices[VertexIndex].Level = Level;
			}
		}

		int32 GetVertexColor(int32 VertexIndex) const
		{
			if (VertexIndex != INDEX_NONE)
			{
				return Vertices[VertexIndex].Color;
			}
			return INDEX_NONE;
		}

		void ReserveVertices(int32 InNumVertices)
		{
			Vertices.Reserve(InNumVertices);
		}

		void ReserveEdges(int32 InNumEdges)
		{
			Edges.Reserve(InNumEdges);
		}

		int32 AddVertex()
		{
			return Vertices.AddDefaulted();
		}

		int32 AddVertices(int32 InNumVertices)
		{
			Vertices.SetNum(Vertices.Num() + InNumVertices, EAllowShrinking::No);
			return Vertices.Num() - InNumVertices;
		}

		int32 AddEdge(int32 VertexIndex0, int32 VertexIndex1)
		{
			int32 EdgeIndex = Edges.Emplace(FEdge(VertexIndex0, VertexIndex1));
			Vertices[VertexIndex0].EdgeIndices.Add(EdgeIndex);
			Vertices[VertexIndex1].EdgeIndices.Add(EdgeIndex);
			return EdgeIndex;
		}

		/**
		 * Assign each vertex to an island. Vertcies in different islands are isolated from each other - thee
		 * is no path which connectes them.
		 */
		void Islandize();

		/**
		 * Assign a "level" to all vertices. Level is the distance from a Vertex with Level = 0 (Level 0 vertices
		 * are specified by the user with SetVertexLevel)
		 */
		void Levelize();

		/**
		 * Assign a color to all vertices. Vertices of the same color do not share any edges (i.e., are not directly connected).
		 */
		void Colorize();

	private:
		struct FVertex
		{
			FVertex() : Island(INDEX_NONE), Level(INDEX_NONE), Color(INDEX_NONE) {}

			// @todo(ccaulfield): eliminate allocations
			TArray<int32> EdgeIndices;
			int32 Island;
			int32 Level;
			int32 Color;
		};

		struct FEdge
		{
			FEdge(int32 VertexIndex0, int32 VertexIndex1)
			{
				VertexIndices[0] = VertexIndex0;
				VertexIndices[1] = VertexIndex1;
			}

			int32 VertexIndices[2];
		};

		struct FIsland
		{
			FIsland() : NextColorIndex(0) {}

			// @todo(ccaulfield): eliminate allocations
			TArray<int32> VertexIndices;
			int32 NextColorIndex;
		};

		// Move vertices from one island into the other
		void MergeIslands(int32 IslandIndex0, int32 IslandIndex1);

		// Move all non-empty islands to lowest island indices and clip array
		void PackIslands();

		// Set the color on the vertex so it is not the same as any adjacent vertices
		void AssignColor(int32 VertexIndex);

		TArray<FVertex> Vertices;
		TArray<FEdge> Edges;
		TArray<FIsland> Islands;
	};

}