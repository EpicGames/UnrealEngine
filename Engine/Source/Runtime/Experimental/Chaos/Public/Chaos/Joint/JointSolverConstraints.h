// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "Chaos/PBDJointConstraintTypes.h"
#include "Chaos/PBDJointConstraintUtilities.h"
#include "Chaos/Utilities.h"

namespace Chaos
{
	class FPBDJointSolver;

	// The method used to calculate the axis and angle for one of the rotation constraints in the joint
	enum class EJointSolverConstraintUpdateType
	{
		None,
		Linear_Point,
		Linear_Spherical,
		Linear_Cylindrical,
		Linear_Planar,

		Linear_SphericalDrive,
		Linear_CircularDrive,
		Linear_AxialDrive,

		Angular_Twist,
		Angular_Cone,
		Angular_SingleLockedSwing,
		Angular_SingleLimitedSwing,
		Angular_DualConeSwing,
		Angular_Locked,

		Angular_TwistDrive,
		Angular_ConeDrive,
		Angular_SwingDrive,
		Angular_SLerpDrive,
	};

	/**
	 * Body and joint state required during a joint's solve (used by all sub-constraints in the joint)
	 */
	class FJointSolverJointState
	{
	public:
		static const int32 MaxConstrainedBodies = 2;

		void Init(
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings,
			const FVec3& PrevP0,
			const FRotation3& PrevQ0,
			const FVec3& PrevP1,
			const FRotation3& PrevQ1,
			const FReal InvM0,
			const FVec3& InvIL0,
			const FReal InvM1,
			const FVec3& InvIL1,
			const FRigidTransform3& XL0,
			const FRigidTransform3& XL1);

		FORCEINLINE void Update(
			const FVec3& P0,
			const FRotation3& Q0,
			const FVec3& P1,
			const FRotation3& Q1);

		FORCEINLINE void InitDerivedState();

		FORCEINLINE void UpdateDerivedState();

		FORCEINLINE void ApplyDelta(
			const FVec3& DP0,
			const FVec3& DR0,
			const FVec3& DP1,
			const FVec3& DR1);

		FORCEINLINE void ApplyRotationDelta(
			const FVec3& DR0,
			const FVec3& DR1);

		// Local-space constraint settings
		FRigidTransform3 XLs[MaxConstrainedBodies];	// Local-space joint connector transforms
		FVec3 InvILs[MaxConstrainedBodies];			// Local-space inverse inertias
		FReal InvMs[MaxConstrainedBodies];			// Inverse masses

		// World-space constraint state
		FVec3 Xs[MaxConstrainedBodies];				// World-space joint connector positions
		FRotation3 Rs[MaxConstrainedBodies];		// World-space joint connector rotations

		// World-space body state
		FVec3 Ps[MaxConstrainedBodies];				// World-space particle CoM positions
		FRotation3 Qs[MaxConstrainedBodies];		// World-space particle CoM rotations
		FMatrix33 InvIs[MaxConstrainedBodies];		// World-space inverse inertias

		// XPBD initial world-space body state (start of each tick, not each sub-tick iteration)
		FVec3 PrevPs[MaxConstrainedBodies];			// World-space particle CoM positions
		FRotation3 PrevQs[MaxConstrainedBodies];	// World-space particle CoM rotations
		FVec3 PrevXs[MaxConstrainedBodies];			// World-space joint connector positions

		FVec3 DPs[MaxConstrainedBodies];
		FVec3 DRs[MaxConstrainedBodies];

		FReal PositionTolerance;					// Distance error below which we consider a constraint or drive solved
		FReal AngleTolerance;						// Angle error below which we consider a constraint or drive solved
	};

	/**
	 * Transient state for a single sub-constraint in a joint
	 */
	class FJointSolverConstraintRowState
	{
	public:
		static const int32 MaxConstrainedBodies = 2;

		FJointSolverConstraintRowState()
			: DPs{ FVec3(0), FVec3(0) }
			, DRs{ FVec3(0), FVec3(0) }
			, Axis(FVec3(0))
			, Error(0)
			, Lambda(0)
		{
		}

		// Reset state that persists over all iterations, but must be cleared at start of each tick.
		FORCEINLINE void TickReset();

		// Reset calculated values ready for next iteration. Note: Lambda is not reset here,
		// it accumulates over the whole timestep.
		FORCEINLINE void IterationReset();

		// Calculate the error to correct, given the current position and limit
		FORCEINLINE void CalculateError(FReal Position, FReal Limit);

		// Per-Iteration Transient
		FVec3 DPs[MaxConstrainedBodies];
		FVec3 DRs[MaxConstrainedBodies];
		FVec3 Axis;
		FReal Error;

		// Per-Tick Transient
		FReal Lambda;
	};

	/**
	 * A single sub-constraint in a Joint (e.g., Twist, Swing, etc.)
	 */
	class FJointSolverConstraintRowData
	{
	public:
		static const int32 MaxConstrainedBodies = 2;

		FJointSolverConstraintRowData()
			: UpdateType(EJointSolverConstraintUpdateType::None)
			, JointIndex(INDEX_NONE)
			, ConstraintIndex(INDEX_NONE)
			, NumRows(0)
			, Stiffness(0)
			, Damping(0)
			, Limit(0)
			, bIsAccelerationMode(false)
			, bIsSoft(false)
		{
		}

		EJointSolverConstraintUpdateType UpdateType;	// Method to use to calculate axis, errors, etc.
		int32 JointIndex;			// Index into the outer container's joint array
		int32 ConstraintIndex;		// Context dependent: X,Y,Z or Twist,Swing1,Swing2 for linear and angular rows
		int32 NumRows;				// Number of rows (including this one) to solve simultaneously
		FReal Stiffness;			// PBD or XPBD stiffness
		FReal Damping;				// XPBD damping
		FReal Limit;
		bool bIsAccelerationMode;
		bool bIsSoft;
	};


	/**
	 * All the constraints in a joint
	 */
	class FJointSolverConstraints
	{
	public:
		friend class FPBDJointSolver;

		FJointSolverConstraints();

		FORCEINLINE void SetJointIndex(int32 InJointIndex)
		{
			JointIndex = InJointIndex;
		}

		FORCEINLINE int32 GetJointIndex() const
		{
			return JointIndex;
		}

		FORCEINLINE int32 GetLinearRowIndexBegin() const
		{
			return LinearRowIndexBegin;
		}

		FORCEINLINE int32 GetLinearRowIndexEnd() const
		{
			return LinearRowIndexEnd;
		}

		FORCEINLINE int32 GetAngularRowIndexBegin() const
		{
			return AngularRowIndexBegin;
		}

		FORCEINLINE int32 GetAngularRowIndexEnd() const
		{
			return AngularRowIndexEnd;
		}

		FORCEINLINE int32 NumLinearConstraints() const 
		{
			return LinearRowIndexEnd - LinearRowIndexBegin;
		}

		FORCEINLINE int32 NumAngularConstraints() const
		{
			return AngularRowIndexEnd - AngularRowIndexBegin;
		}

		void AddPositionConstraints(
			TArray<FJointSolverConstraintRowData>& RowDatas,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		void AddRotationConstraints(
			TArray<FJointSolverConstraintRowData>& RowDatas,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		// Update the axes and positions of all position constraints
		void UpdatePositionConstraints(
			const TArray<FJointSolverConstraintRowData>& RowDatas,
			TArray<FJointSolverConstraintRowState>& RowStates,
			const FJointSolverJointState& JointState,
			const FPBDJointSettings& JointSettings);

		// Update the axes and positions of all rotation constraints
		void UpdateRotationConstraints(
			const TArray<FJointSolverConstraintRowData>& RowDatas,
			TArray<FJointSolverConstraintRowState>& RowStates,
			const FJointSolverJointState& JointState,
			const FPBDJointSettings& JointSettings);

	private:

		void AddPointPositionConstraint(
			TArray<FJointSolverConstraintRowData>& RowDatas,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		void AddSphericalPositionConstraint(
			TArray<FJointSolverConstraintRowData>& RowDatas,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		void AddCylindricalPositionConstraint(
			TArray<FJointSolverConstraintRowData>& RowDatas,
			const int32 AxisIndex,
			const EJointMotionType AxialMotion,
			const EJointMotionType RadialMotion,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		void AddPlanarPositionConstraint(
			TArray<FJointSolverConstraintRowData>& RowDatas,
			const int32 AxisIndex,
			const EJointMotionType AxialMotion,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		void AddPositionDrive(
			TArray<FJointSolverConstraintRowData>& RowDatas,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings,
			const EJointSolverConstraintUpdateType UpdateType,
			const int32 AxisIndex);

		void AddTwistConstraint(
			TArray<FJointSolverConstraintRowData>& RowDatas,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings,
			const EJointSolverConstraintUpdateType Type,
			const bool bUseSoftLimit);

		void AddSwingConstraint(
			TArray<FJointSolverConstraintRowData>& RowDatas,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings,
			const EJointSolverConstraintUpdateType Type,
			const EJointAngularConstraintIndex ConstraintIndex,
			const bool bUseSoftLimit);

		void AddTwistDrive(
			TArray<FJointSolverConstraintRowData>& RowDatas,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		void AddSwingDrive(
			TArray<FJointSolverConstraintRowData>& RowDatas,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings,
			const EJointSolverConstraintUpdateType Type,
			const EJointAngularConstraintIndex ConstraintIndex);

		void AddSLerpDrive(
			TArray<FJointSolverConstraintRowData>& RowDatas,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);



		FORCEINLINE int32 UpdatePointPositionConstraint(
			int32 RowIndex,
			const TArray<FJointSolverConstraintRowData>& RowDatas,
			TArray<FJointSolverConstraintRowState>& RowStates,
			const FPBDJointSettings& JointSettings,
			const FVec3& X0,
			const FVec3& X1);

		FORCEINLINE int32 UpdateSphericalPositionConstraint(
			int32 RowIndex,
			const TArray<FJointSolverConstraintRowData>& RowDatas,
			TArray<FJointSolverConstraintRowState>& RowStates,
			const FPBDJointSettings& JointSettings,
			const FVec3& X0,
			const FVec3& X1);

		FORCEINLINE int32 UpdateCylindricalPositionConstraint(
			int32 RowIndex,
			const TArray<FJointSolverConstraintRowData>& RowDatas,
			TArray<FJointSolverConstraintRowState>& RowStates,
			const FPBDJointSettings& JointSettings,
			const FRotation3& R0,
			const FVec3& X0,
			const FVec3& X1);

		FORCEINLINE int32 UpdatePlanarPositionConstraint(
			int32 RowIndex,
			const TArray<FJointSolverConstraintRowData>& RowDatas,
			TArray<FJointSolverConstraintRowState>& RowStates,
			const FPBDJointSettings& JointSettings,
			const FRotation3& R0,
			const FVec3& X0,
			const FVec3& X1);

		FORCEINLINE int32 UpdateSphericalPositionDrive(
			int32 RowIndex,
			const TArray<FJointSolverConstraintRowData>& RowDatas,
			TArray<FJointSolverConstraintRowState>& RowStates,
			const FPBDJointSettings& JointSettings,
			const FRotation3& R0,
			const FVec3& X0,
			const FVec3& X1);

		FORCEINLINE int32 UpdateCircularPositionDrive(
			int32 RowIndex,
			const TArray<FJointSolverConstraintRowData>& RowDatas,
			TArray<FJointSolverConstraintRowState>& RowStates,
			const FPBDJointSettings& JointSettings,
			const FRotation3& R0,
			const FVec3& X0,
			const FVec3& X1);

		FORCEINLINE int32 UpdateAxialPositionDrive(
			int32 RowIndex,
			const TArray<FJointSolverConstraintRowData>& RowDatas,
			TArray<FJointSolverConstraintRowState>& RowStates,
			const FPBDJointSettings& JointSettings,
			const FRotation3& R0,
			const FVec3& X0,
			const FVec3& X1);

		FORCEINLINE void UpdateTwistConstraint(
			const FJointSolverConstraintRowData& RowData,
			FJointSolverConstraintRowState& RowState,
			const FPBDJointSettings& JointSettings,
			const FRotation3& R1,
			const FRotation3& RTwist);

		FORCEINLINE void UpdateConeSwingConstraint(
			const FJointSolverConstraintRowData& RowData,
			FJointSolverConstraintRowState& RowState,
			const FPBDJointSettings& JointSettings,
			const FRotation3& R0,
			const FRotation3& RSwing);

		FORCEINLINE void UpdateSingleLockedSwingConstraint(
			const FJointSolverConstraintRowData& RowData,
			FJointSolverConstraintRowState& RowState,
			const FPBDJointSettings& JointSettings,
			const FRotation3& R0,
			const FRotation3& R1);

		FORCEINLINE void UpdateSingleLimitedSwingConstraint(
			const FJointSolverConstraintRowData& RowData,
			FJointSolverConstraintRowState& RowState,
			const FPBDJointSettings& JointSettings,
			const FRotation3& R0,
			const FRotation3& RSwing);

		FORCEINLINE void UpdateDualConeSwingConstraint(
			const FJointSolverConstraintRowData& RowData,
			FJointSolverConstraintRowState& RowState,
			const FPBDJointSettings& JointSettings,
			const FRotation3& R0,
			const FRotation3& R1);

		FORCEINLINE void UpdateTwistDrive(
			const FJointSolverConstraintRowData& RowData,
			FJointSolverConstraintRowState& RowState,
			const FPBDJointSettings& JointSettings,
			const FRotation3& R1,
			const FRotation3& RTwist);

		FORCEINLINE void UpdateConeSwingDrive(
			const FJointSolverConstraintRowData& RowData,
			FJointSolverConstraintRowState& RowState,
			const FPBDJointSettings& JointSettings,
			const FRotation3& R0,
			const FRotation3& RSwing);

		FORCEINLINE void UpdateSwingDrive(
			const FJointSolverConstraintRowData& RowData,
			FJointSolverConstraintRowState& RowState,
			const FPBDJointSettings& JointSettings,
			const FRotation3& R0,
			const FRotation3& RSwing);

		FORCEINLINE void UpdateSLerpDrive(
			const FJointSolverConstraintRowData& RowData,
			FJointSolverConstraintRowState& RowState,
			const FPBDJointSettings& JointSettings,
			const FRotation3& R0,
			const FRotation3& R1);

		int32 JointIndex;
		int32 LinearRowIndexBegin;
		int32 LinearRowIndexEnd;
		int32 AngularRowIndexBegin;
		int32 AngularRowIndexEnd;
		FReal PositionTolerance;
		FReal AngleTolerance;
		bool bNeedSwingTwist;	// True if any of the constraints need the decomposed swing/twist to calculate angles
		bool bNeedLockedAxes;	// True if any of the constraints need the fixed axes to calculate angles
	};

	class FJointSolver
	{
	public:
		static int32 ApplyPositionConstraints(
			const FReal Dt,
			TArray<FJointSolverJointState>& JointStates,
			const TArray <FJointSolverConstraintRowData>& RowDatas,
			TArray <FJointSolverConstraintRowState>& RowStates,
			int32 JointIndexBegin,
			int32 JointIndexEnd,
			int32 RowIndexBegin,
			int32 RowIndexEnd);

		static int32 ApplyRotationConstraints(
			const FReal Dt,
			TArray<FJointSolverJointState>& JointStates,
			const TArray <FJointSolverConstraintRowData>& RowDatas,
			TArray <FJointSolverConstraintRowState>& RowStates,
			int32 JointIndexBegin,
			int32 JointIndexEnd,
			int32 RowIndexBegin,
			int32 RowIndexEnd);

	private:
		static FORCEINLINE void ApplyPositionConstraint1(
			const FReal Dt,
			const FJointSolverJointState& JointState,
			const FJointSolverConstraintRowData& RowData,
			FJointSolverConstraintRowState& RowState);

		static FORCEINLINE void ApplyPositionConstraint3(
			const FReal Dt,
			const FJointSolverJointState& JointState,
			const FJointSolverConstraintRowData& RowData,
			FJointSolverConstraintRowState& RowState0,
			FJointSolverConstraintRowState& RowState1,
			FJointSolverConstraintRowState& RowState2);

		static FORCEINLINE void ApplyRotationConstraint(
			const FReal Dt,
			const FJointSolverJointState& JointState,
			const FJointSolverConstraintRowData& RowData,
			FJointSolverConstraintRowState& RowState);
	};
}

#include "Chaos/Joint/JointSolverConstraints.inl" // IWYU pragma: export
