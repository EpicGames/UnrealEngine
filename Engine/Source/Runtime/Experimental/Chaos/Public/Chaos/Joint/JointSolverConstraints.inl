// Copyright Epic Games, Inc. All Rights Reserved.

namespace Chaos
{
	FORCEINLINE void FJointSolverJointState::Init(
		const FPBDJointSolverSettings& SolverSettings,
		const FPBDJointSettings& JointSettings,
		const FVec3& PrevP0,
		const FRotation3& PrevQ0,
		const FVec3& PrevP1,
		const FRotation3& PrevQ1,
		const FReal InvM0,
		const FVec3& InvIL0,
		const FReal InvM1,
		const FVec3& InvIL1,
		const FRigidTransform3& XL0,
		const FRigidTransform3& XL1)
	{
		XLs[0] = XL0;
		XLs[1] = XL1;

		InvILs[0] = JointSettings.ParentInvMassScale * InvIL0;
		InvILs[1] = InvIL1;
		InvMs[0] = JointSettings.ParentInvMassScale * InvM0;
		InvMs[1] = InvM1;

		FPBDJointUtilities::ConditionInverseMassAndInertia(InvMs[0], InvMs[1], InvILs[0], InvILs[1], SolverSettings.MinParentMassRatio, SolverSettings.MaxInertiaRatio, InvMs[0], InvMs[1], InvILs[0], InvILs[1]);

		PrevPs[0] = PrevP0;
		PrevPs[1] = PrevP1;
		PrevQs[0] = PrevQ0;
		PrevQs[1] = PrevQ1;
		PrevXs[0] = PrevP0 + PrevQ0 * XL0.GetTranslation();
		PrevXs[1] = PrevP1 + PrevQ1 * XL1.GetTranslation();

		PositionTolerance = SolverSettings.PositionTolerance;
		AngleTolerance = SolverSettings.AngleTolerance;

		InitDerivedState();
	}

	FORCEINLINE void FJointSolverJointState::Update(
		const FVec3& P0,
		const FRotation3& Q0,
		const FVec3& P1,
		const FRotation3& Q1)
	{
		Ps[0] = P0;
		Ps[1] = P1;
		Qs[0] = Q0;
		Qs[1] = Q1;
		Qs[1].EnforceShortestArcWith(Qs[0]);
	}

	FORCEINLINE void FJointSolverJointState::InitDerivedState()
	{
		// Really we should need to do this for Kinematics since Dynamics are updated each iteration
		Xs[0] = PrevPs[0] + PrevQs[0] * XLs[0].GetTranslation();
		Rs[0] = PrevQs[0] * XLs[0].GetRotation();
		InvIs[0] = (InvMs[0] > 0.0f) ? Utilities::ComputeWorldSpaceInertia(PrevQs[0], InvILs[0]) : FMatrix33(0, 0, 0);

		Xs[1] = PrevPs[1] + PrevQs[1] * XLs[1].GetTranslation();
		Rs[1] = PrevQs[1] * XLs[1].GetRotation();
		InvIs[1] = (InvMs[1] > 0.0f) ? Utilities::ComputeWorldSpaceInertia(PrevQs[1], InvILs[1]) : FMatrix33(0, 0, 0);

		Rs[1].EnforceShortestArcWith(Rs[0]);

		PrevXs[0] = Xs[0];
		PrevXs[1] = Xs[1];

		DPs[0] = FVec3(0);
		DRs[0] = FVec3(0);
		DPs[1] = FVec3(0);
		DRs[1] = FVec3(0);
	}

	FORCEINLINE void FJointSolverJointState::UpdateDerivedState()
	{
		if (InvMs[0] > 0.0f)
		{
			Xs[0] = Ps[0] + Qs[0] * XLs[0].GetTranslation();
			Rs[0] = Qs[0] * XLs[0].GetRotation();
			InvIs[0] = Utilities::ComputeWorldSpaceInertia(Qs[0], InvILs[0]);
			DPs[0] = FVec3(0);
			DRs[0] = FVec3(0);
		}
		if (InvMs[1] > 0.0f)
		{
			Xs[1] = Ps[1] + Qs[1] * XLs[1].GetTranslation();
			Rs[1] = Qs[1] * XLs[1].GetRotation();
			InvIs[1] = Utilities::ComputeWorldSpaceInertia(Qs[1], InvILs[1]);
			DPs[1] = FVec3(0);
			DRs[1] = FVec3(0);
		}
		Rs[1].EnforceShortestArcWith(Rs[0]);
	}

	FORCEINLINE void FJointSolverJointState::ApplyDelta(
		const FVec3& DP0,
		const FVec3& DR0,
		const FVec3& DP1,
		const FVec3& DR1)
	{
		if (InvMs[0] > 0.0f)
		{
			//UE_LOGF(LogChaosJoint, VeryVerbose, "      Apply DP%d %f %f %f", 0, DP0.X, DP0.Y, DP0.Z);
			//UE_LOGF(LogChaosJoint, VeryVerbose, "      Apply DR%d %f %f %f", 0, DR0.X, DR0.Y, DR0.Z);

			Ps[0] += DP0;
			const FRotation3 DQ0 = (FRotation3::FromElements(DR0, 0) * Qs[0]) * (FReal)0.5;
			Qs[0] = (Qs[0] + DQ0).GetNormalized();
		}
		if (InvMs[1] > 0.0f)
		{
			//UE_LOGF(LogChaosJoint, VeryVerbose, "      Apply DP%d %f %f %f", 1, DP1.X, DP1.Y, DP1.Z);
			//UE_LOGF(LogChaosJoint, VeryVerbose, "      Apply DR%d %f %f %f", 1, DR1.X, DR1.Y, DR1.Z);

			Ps[1] += DP1;
			const FRotation3 DQ1 = (FRotation3::FromElements(DR1, 0) * Qs[1]) * (FReal)0.5;
			Qs[1] = (Qs[1] + DQ1).GetNormalized();
		}
		Qs[1].EnforceShortestArcWith(Qs[0]);
	}

	FORCEINLINE void FJointSolverJointState::ApplyRotationDelta(
		const FVec3& DR0,
		const FVec3& DR1)
	{
		if (InvMs[0] > 0.0f)
		{
			//UE_LOGF(LogChaosJoint, VeryVerbose, "      Apply DR%d %f %f %f", 0, DR0.X, DR0.Y, DR0.Z);

			const FRotation3 DQ0 = (FRotation3::FromElements(DR0, 0) * Qs[0]) * (FReal)0.5;
			Qs[0] = (Qs[0] + DQ0).GetNormalized();
		}
		if (InvMs[1] > 0.0f)
		{
			//UE_LOGF(LogChaosJoint, VeryVerbose, "      Apply DR%d %f %f %f", 1, DR1.X, DR1.Y, DR1.Z);

			const FRotation3 DQ1 = (FRotation3::FromElements(DR1, 0) * Qs[1]) * (FReal)0.5;
			Qs[1] = (Qs[1] + DQ1).GetNormalized();
		}
		Qs[1].EnforceShortestArcWith(Qs[0]);
	}

	//
	///////////////////////////////////////////////////////////////////////
	//

	FORCEINLINE void FJointSolverConstraintRowState::TickReset()
	{
		Lambda = 0.0f;
	}

	// Reset calculated values ready for next iteration. Note: Lambda is not reset here,
	// it accumulates over the whole timestep.
	FORCEINLINE void FJointSolverConstraintRowState::IterationReset()
	{
		DPs[0] = FVec3(0);
		DPs[1] = FVec3(0);
		DRs[0] = FVec3(0);
		DRs[1] = FVec3(0);
		Axis = FVec3(0);
		Error = 0.0f;
		// Do not reset Lambda
	}

	FORCEINLINE void FJointSolverConstraintRowState::CalculateError(FReal Position, FReal Limit)
	{
		Error = 0.0f;
		if (Position >= Limit)
		{
			Error = Position - Limit;
		}
		else if (Position <= -Limit)
		{
			Error = Position + Limit;
		}
	}

	//
	//////////////////////////////////////////////////////////////////////////
	//


	FORCEINLINE int32 FJointSolverConstraints::UpdatePointPositionConstraint(
		int32 RowIndexBegin,
		const TArray<FJointSolverConstraintRowData>& RowDatas,
		TArray<FJointSolverConstraintRowState>& RowStates,
		const FPBDJointSettings& JointSettings,
		const FVec3& X0,
		const FVec3& X1)
	{
		const FVec3 CX = X1 - X0;
		for (int32 AxisIndex = 0; AxisIndex < 3; ++AxisIndex)
		{
			FVec3 Axis = FVec3(0.0f);
			Axis[AxisIndex] = 1.0f;
			RowStates[RowIndexBegin + AxisIndex].Axis = Axis;
			RowStates[RowIndexBegin + AxisIndex].CalculateError(CX[AxisIndex], RowDatas[RowIndexBegin + AxisIndex].Limit);
		}

		return 3;
	}

	FORCEINLINE int32 FJointSolverConstraints::UpdateSphericalPositionConstraint(
		int32 RowIndexBegin,
		const TArray<FJointSolverConstraintRowData>& RowDatas,
		TArray<FJointSolverConstraintRowState>& RowStates,
		const FPBDJointSettings& JointSettings,
		const FVec3& X0,
		const FVec3& X1)
	{
		const FVec3 CX = X1 - X0;
		const FReal CXLen = CX.Size();
		if (CXLen > UE_KINDA_SMALL_NUMBER)
		{
			RowStates[RowIndexBegin].Axis = CX / CXLen;
			RowStates[RowIndexBegin].CalculateError(CXLen, RowDatas[RowIndexBegin].Limit);
		}

		return 1;
	}

	FORCEINLINE int32 FJointSolverConstraints::UpdateCylindricalPositionConstraint(
		int32 RowIndexBegin,
		const TArray<FJointSolverConstraintRowData>& RowDatas,
		TArray<FJointSolverConstraintRowState>& RowStates,
		const FPBDJointSettings& JointSettings,
		const FRotation3& R0,
		const FVec3& X0,
		const FVec3& X1)
	{
		const FJointSolverConstraintRowData& AxisData = RowDatas[RowIndexBegin + 0];
		const FJointSolverConstraintRowData& RadialData = RowDatas[RowIndexBegin + 1];
		check(RadialData.UpdateType == EJointSolverConstraintUpdateType::None);

		FVec3 Axis, RadialAxis;
		FReal AxialDelta, RadialDelta;
		FPBDJointUtilities::GetCylindricalAxesDeltas(R0, X0, X1, AxisData.ConstraintIndex, Axis, AxialDelta, RadialAxis, RadialDelta);

		FJointSolverConstraintRowState& AxisState = RowStates[RowIndexBegin + 0];
		AxisState.Axis = Axis;
		AxisState.CalculateError(AxialDelta, AxisData.Limit);

		FJointSolverConstraintRowState& RadialState = RowStates[RowIndexBegin + 1];
		RadialState.Axis = RadialAxis;
		RadialState.CalculateError(RadialDelta, RadialData.Limit);

		return 2;
	}

	FORCEINLINE int32 FJointSolverConstraints::UpdatePlanarPositionConstraint(
		int32 RowIndexBegin,
		const TArray<FJointSolverConstraintRowData>& RowDatas,
		TArray<FJointSolverConstraintRowState>& RowStates,
		const FPBDJointSettings& JointSettings,
		const FRotation3& R0,
		const FVec3& X0,
		const FVec3& X1)
	{
		const FJointSolverConstraintRowData& RowData = RowDatas[RowIndexBegin];

		FVec3 Axis;
		FReal Delta;
		FPBDJointUtilities::GetPlanarAxisDelta(R0, X0, X1, RowData.ConstraintIndex, Axis, Delta);

		FJointSolverConstraintRowState& RowState = RowStates[RowIndexBegin];
		RowState.Axis = Axis;
		RowState.CalculateError(Delta, RowData.Limit);

		return 1;
	}

	FORCEINLINE int32 FJointSolverConstraints::UpdateSphericalPositionDrive(
		int32 RowIndex,
		const TArray<FJointSolverConstraintRowData>& RowDatas,
		TArray<FJointSolverConstraintRowState>& RowStates,
		const FPBDJointSettings& JointSettings,
		const FRotation3& R0,
		const FVec3& X0,
		const FVec3& X1)
	{
		const FJointSolverConstraintRowData& RowData = RowDatas[RowIndex];
		FJointSolverConstraintRowState& RowState = RowStates[RowIndex];

		const FVec3 XTarget = X0 + R0 * JointSettings.LinearDrivePositionTarget;
		FVec3 Axis;
		FReal Delta;
		FPBDJointUtilities::GetSphericalAxisDelta(XTarget, X1, Axis, Delta);

		RowState.Axis = Axis;
		RowState.CalculateError(Delta, RowData.Limit);

		return 1;
	}

	FORCEINLINE int32 FJointSolverConstraints::UpdateCircularPositionDrive(
		int32 RowIndex,
		const TArray<FJointSolverConstraintRowData>& RowDatas,
		TArray<FJointSolverConstraintRowState>& RowStates,
		const FPBDJointSettings& JointSettings,
		const FRotation3& R0,
		const FVec3& X0,
		const FVec3& X1)
	{
		const FJointSolverConstraintRowData& RowData = RowDatas[RowIndex];
		FJointSolverConstraintRowState& RowState = RowStates[RowIndex];

		const FVec3 XTarget = X0 + R0 * JointSettings.LinearDrivePositionTarget;
		FVec3 Axis, RadialAxis;
		FReal AxialDelta, RadialDelta;
		FPBDJointUtilities::GetCylindricalAxesDeltas(R0, XTarget, X1, RowData.ConstraintIndex, Axis, AxialDelta, RadialAxis, RadialDelta);

		RowState.Axis = RadialAxis;
		RowState.CalculateError(RadialDelta, RowData.Limit);

		return 1;
	}

	FORCEINLINE int32 FJointSolverConstraints::UpdateAxialPositionDrive(
		int32 RowIndex,
		const TArray<FJointSolverConstraintRowData>& RowDatas,
		TArray<FJointSolverConstraintRowState>& RowStates,
		const FPBDJointSettings& JointSettings,
		const FRotation3& R0,
		const FVec3& X0,
		const FVec3& X1)
	{
		const FJointSolverConstraintRowData& RowData = RowDatas[RowIndex];
		FJointSolverConstraintRowState& RowState = RowStates[RowIndex];

		const FVec3 XTarget = X0 + R0 * JointSettings.LinearDrivePositionTarget;
		FVec3 Axis;
		FReal Delta;
		FPBDJointUtilities::GetPlanarAxisDelta(R0, XTarget, X1, RowData.ConstraintIndex, Axis, Delta);

		RowState.Axis = Axis;
		RowState.CalculateError(Delta, RowData.Limit);

		return 1;
	}

	//
	//////////////////////////////////////////////////////////////////////////
	//

	FORCEINLINE void FJointSolverConstraints::UpdateTwistConstraint(
		const FJointSolverConstraintRowData& RowData,
		FJointSolverConstraintRowState& RowState,
		const FPBDJointSettings& JointSettings,
		const FRotation3& R1,
		const FRotation3& RTwist)
	{
		RowState.Axis = R1 * FJointConstants::TwistAxis();
		RowState.CalculateError(FPBDJointUtilities::GetTwistAngle(RTwist), RowData.Limit);
	}

	FORCEINLINE void FJointSolverConstraints::UpdateConeSwingConstraint(
		const FJointSolverConstraintRowData& RowData,
		FJointSolverConstraintRowState& RowState,
		const FPBDJointSettings& JointSettings,
		const FRotation3& R0,
		const FRotation3& RSwing)
	{
		FReal SwingAngle;
		FVec3 SwingAxisLocal;
		RSwing.ToAxisAndAngleSafe(SwingAxisLocal, SwingAngle, FJointConstants::Swing1Axis(), 1.e-6f);
		if (SwingAngle > UE_PI)
		{
			SwingAngle = SwingAngle - (FReal)2 * UE_PI;
		}

		RowState.Axis = R0 * SwingAxisLocal;
		RowState.CalculateError(SwingAngle, FPBDJointUtilities::GetConeAngleLimit(JointSettings, SwingAxisLocal, SwingAngle));
	}

	FORCEINLINE void FJointSolverConstraints::UpdateSingleLockedSwingConstraint(
		const FJointSolverConstraintRowData& RowData,
		FJointSolverConstraintRowState& RowState,
		const FPBDJointSettings& JointSettings,
		const FRotation3& R0,
		const FRotation3& R1)
	{
		FVec3 Twist1 = R1 * FJointConstants::TwistAxis();
		FVec3 Swing0 = R0 * FJointConstants::OtherSwingAxis((EJointAngularConstraintIndex)RowData.ConstraintIndex);

		RowState.Axis = FVec3::CrossProduct(Swing0, Twist1);
		RowState.CalculateError(-FVec3::DotProduct(Swing0, Twist1), RowData.Limit);
	}

	FORCEINLINE void FJointSolverConstraints::UpdateSingleLimitedSwingConstraint(
		const FJointSolverConstraintRowData& RowData,
		FJointSolverConstraintRowState& RowState,
		const FPBDJointSettings& JointSettings,
		const FRotation3& R0,
		const FRotation3& RSwing)
	{
		const FReal R01SwingYorZ = (FJointConstants::AxisIndex((EJointAngularConstraintIndex)RowData.ConstraintIndex) == 2) ? RSwing.Z : RSwing.Y;	// Can't index a quat :(
		const FReal Angle = (FReal)4. * FMath::Atan2(R01SwingYorZ, (FReal)(1. + RSwing.W));
		const FVec3& AxisLocal = (RowData.ConstraintIndex == (int32)EJointAngularConstraintIndex::Swing1) ? FJointConstants::Swing1Axis() : FJointConstants::Swing2Axis();

		RowState.Axis = R0 * AxisLocal;
		RowState.CalculateError(Angle, RowData.Limit);
	}

	FORCEINLINE void FJointSolverConstraints::UpdateDualConeSwingConstraint(
		const FJointSolverConstraintRowData& RowData,
		FJointSolverConstraintRowState& RowState,
		const FPBDJointSettings& JointSettings,
		const FRotation3& R0,
		const FRotation3& R1)
	{
		FVec3 Twist1 = R1 * FJointConstants::TwistAxis();
		FVec3 Swing0 = R0 * FJointConstants::OtherSwingAxis((EJointAngularConstraintIndex)RowData.ConstraintIndex);

		FVec3 Axis = FVec3::CrossProduct(Swing0, Twist1);
		if (Utilities::NormalizeSafe(Axis, UE_KINDA_SMALL_NUMBER))
		{
			FReal SwingTwistDot = FVec3::DotProduct(Swing0, Twist1);
			FReal Position = FMath::Asin(FMath::Clamp(-SwingTwistDot, FReal(-1), FReal(1)));
			RowState.Axis = Axis;
			RowState.CalculateError(Position, RowData.Limit);
		}
	}

	FORCEINLINE void FJointSolverConstraints::UpdateTwistDrive(
		const FJointSolverConstraintRowData& RowData,
		FJointSolverConstraintRowState& RowState,
		const FPBDJointSettings& JointSettings,
		const FRotation3& R1,
		const FRotation3& RTwist)
	{
		// @todo(ccaulfield): Reimplement with AngularDriveTarget
		//const FVec3 Axis = R1 * FJointConstants::TwistAxis();
		//const FReal Angle = FPBDJointUtilities::GetTwistAngle(RTwist);
		//const FReal TwistAngleTarget = JointSettings.AngularDriveTargetAngles[(int32)EJointAngularConstraintIndex::Twist];
		//const FReal DTwistAngle = Angle - TwistAngleTarget;

		//RowState.Axis = Axis;
		//RowState.CalculateError(DTwistAngle, RowData.Limit);
	}

	FORCEINLINE void FJointSolverConstraints::UpdateConeSwingDrive(
		const FJointSolverConstraintRowData& RowData,
		FJointSolverConstraintRowState& RowState,
		const FPBDJointSettings& JointSettings,
		const FRotation3& R0,
		const FRotation3& RSwing)
	{
		// @todo(ccaulfield): Reimplement with AngularDriveTarget
		//FReal SwingAngle;
		//FVec3 SwingAxisLocal;
		//RSwing.ToAxisAndAngleSafe(SwingAxisLocal, SwingAngle, FJointConstants::Swing1Axis(), AngleTolerance);
		//if (SwingAngle > PI)
		//{
		//	SwingAngle = SwingAngle - (FReal)2 * PI;
		//}

		//// Circular swing target (max of Swing1, Swing2 targets)
		//// @todo(ccaulfield): what should cone target really do?
		//const FReal Swing1Target = JointSettings.AngularDriveTargetAngles[(int32)EJointAngularConstraintIndex::Swing1];
		//const FReal Swing2Target = JointSettings.AngularDriveTargetAngles[(int32)EJointAngularConstraintIndex::Swing2];
		//const FReal SwingAngleTarget = FMath::Max(Swing1Target, Swing2Target);
		//const FReal DSwingAngle = SwingAngle - SwingAngleTarget;

		//RowState.Axis = R0 * SwingAxisLocal;
		//RowState.CalculateError(DSwingAngle, RowData.Limit);
	}

	FORCEINLINE void FJointSolverConstraints::UpdateSwingDrive(
		const FJointSolverConstraintRowData& RowData,
		FJointSolverConstraintRowState& RowState,
		const FPBDJointSettings& JointSettings,
		const FRotation3& R0,
		const FRotation3& RSwing)
	{
		// @todo(ccaulfield): Reimplement with AngularDriveTarget
		//const FReal R01SwingYorZ = (FJointConstants::AxisIndex((EJointAngularConstraintIndex)RowData.ConstraintIndex) == 2) ? RSwing.Z : RSwing.Y;	// Can't index a quat :(
		//const FReal SwingAngle = 4.0f * FMath::Atan2(R01SwingYorZ, 1.0f + RSwing.W);
		//const FReal SwingAngleTarget = JointSettings.AngularDriveTargetAngles[(int32)RowData.ConstraintIndex];
		//const FReal DSwingAngle = SwingAngle - SwingAngleTarget;
		//const FVec3& AxisLocal = ((EJointAngularConstraintIndex)RowData.ConstraintIndex == EJointAngularConstraintIndex::Swing1) ? FJointConstants::Swing1Axis() : FJointConstants::Swing2Axis();

		//RowState.Axis = R0 * AxisLocal;
		//RowState.CalculateError(DSwingAngle, RowData.Limit);
	}

	FORCEINLINE void FJointSolverConstraints::UpdateSLerpDrive(
		const FJointSolverConstraintRowData& RowData,
		FJointSolverConstraintRowState& RowState,
		const FPBDJointSettings& JointSettings,
		const FRotation3& R0,
		const FRotation3& R1)
	{
		const FRotation3 TargetR1 = R0 * JointSettings.AngularDrivePositionTarget;
		const FRotation3 DR = TargetR1 * R1.Inverse();

		FVec3 SLerpAxis;
		FReal SLerpAngle;
		if (DR.ToAxisAndAngleSafe(SLerpAxis, SLerpAngle, FVec3(1, 0, 0)))
		{
			if (SLerpAngle > (FReal)UE_PI)
			{
				SLerpAngle = SLerpAngle - (FReal)2 * UE_PI;
			}

			//UE_LOGF(LogChaosJoint, VeryVerbose, "      SLerpDrive Pos: %f Axis: %f %f %f", -SLerpAngle, SLerpAxis.X, SLerpAxis.Y, SLerpAxis.Z);

			RowState.Axis = SLerpAxis;
			RowState.CalculateError(-SLerpAngle, RowData.Limit);
		}
	}

	//
	///////////////////////////////////////////////////////////////////////
	//

	FORCEINLINE void FJointSolver::ApplyPositionConstraint1(
		const FReal Dt,
		const FJointSolverJointState& JointState,
		const FJointSolverConstraintRowData& RowData,
		FJointSolverConstraintRowState& RowState)
	{
		//UE_LOGF(LogChaosJoint, VeryVerbose, "    Position Error %f", RowState.Error);

		FReal II = 0.0f;
		FVec3 AngularAxis0 = FVec3(0);
		FVec3 AngularAxis1 = FVec3(0);

		if (JointState.InvMs[0] > 0.0f)
		{
			AngularAxis0 = FVec3::CrossProduct(JointState.Xs[0] - JointState.Ps[0], RowState.Axis);
			const FVec3 IA0 = Utilities::Multiply(JointState.InvIs[0], AngularAxis0);
			const FReal II0 = FVec3::DotProduct(AngularAxis0, IA0);
			II += JointState.InvMs[0] + II0;
		}
		if (JointState.InvMs[1] > 0.0f)
		{
			AngularAxis1 = FVec3::CrossProduct(JointState.Xs[1] - JointState.Ps[1], RowState.Axis);
			const FVec3 IA1 = Utilities::Multiply(JointState.InvIs[1], AngularAxis1);
			const FReal II1 = FVec3::DotProduct(AngularAxis1, IA1);
			II += JointState.InvMs[1] + II1;
		}

		if (RowData.bIsSoft)
		{
			FReal VelDt = 0;
			if (RowData.Damping > UE_KINDA_SMALL_NUMBER)
			{
				const FVec3 V0 = FVec3::CalculateVelocity(JointState.PrevXs[0], JointState.Xs[0], 1.0f);
				const FVec3 V1 = FVec3::CalculateVelocity(JointState.PrevXs[1], JointState.Xs[1], 1.0f);
				VelDt = FVec3::DotProduct(V0 - V1, RowState.Axis);
			}

			const FReal SpringMassScale = (RowData.bIsAccelerationMode) ? 1.0f / (JointState.InvMs[0] + JointState.InvMs[1]) : 1.0f;
			const FReal S = SpringMassScale * RowData.Stiffness * Dt * Dt;
			const FReal D = SpringMassScale * RowData.Damping * Dt;
			const FReal Multiplier = (FReal)1 / ((S + D) * II + (FReal)1);
			const FReal DLambda = Multiplier * (S * RowState.Error - D * VelDt - RowState.Lambda);

			RowState.DPs[0] = (JointState.InvMs[0] * DLambda) * RowState.Axis;
			RowState.DPs[1] = (-JointState.InvMs[1] * DLambda) * RowState.Axis;
			RowState.DRs[0] = DLambda * Utilities::Multiply(JointState.InvIs[0], AngularAxis0);
			RowState.DRs[1] = -DLambda * Utilities::Multiply(JointState.InvIs[1], AngularAxis1);
			RowState.Lambda += DLambda;
		}
		else
		{
			const FVec3 DX = RowState.Axis * (RowData.Stiffness * RowState.Error / II);

			// Apply constraint correction
			RowState.DPs[0] = JointState.InvMs[0] * DX;
			RowState.DPs[1] = -JointState.InvMs[1] * DX;
			RowState.DRs[0] = Utilities::Multiply(JointState.InvIs[0], FVec3::CrossProduct(JointState.Xs[0] - JointState.Ps[0], DX));
			RowState.DRs[1] = Utilities::Multiply(JointState.InvIs[1], FVec3::CrossProduct(JointState.Xs[1] - JointState.Ps[1], -DX));
		}
	}

	// NOTE: Currently assumes row axes are unit axes
	FORCEINLINE void FJointSolver::ApplyPositionConstraint3(
		const FReal Dt,
		const FJointSolverJointState& JointState,
		const FJointSolverConstraintRowData& RowData,
		FJointSolverConstraintRowState& RowState0,
		FJointSolverConstraintRowState& RowState1,
		FJointSolverConstraintRowState& RowState2)
	{
		FVec3 CX = FVec3(RowState0.Error, RowState1.Error, RowState2.Error);

		//UE_LOGF(LogChaosJoint, VeryVerbose, "    Position Error %f", CX.Size());

		FMatrix33 M = FMatrix33(0, 0, 0);
		if (JointState.InvMs[0] > 0)
		{
			FMatrix33 M0 = Utilities::ComputeJointFactorMatrix(JointState.Xs[0] - JointState.Ps[0], JointState.InvIs[0], JointState.InvMs[0]);
			M += M0;
		}
		if (JointState.InvMs[1] > 0)
		{
			FMatrix33 M1 = Utilities::ComputeJointFactorMatrix(JointState.Xs[1] - JointState.Ps[1], JointState.InvIs[1], JointState.InvMs[1]);
			M += M1;
		}
		FMatrix33 MI = M.Inverse();
		const FVec3 DX = RowData.Stiffness * Utilities::Multiply(MI, CX);

		// Apply constraint correction
		RowState0.DPs[0] = JointState.InvMs[0] * DX;
		RowState0.DPs[1] = -JointState.InvMs[1] * DX;
		RowState0.DRs[0] = Utilities::Multiply(JointState.InvIs[0], FVec3::CrossProduct(JointState.Xs[0] - JointState.Ps[0], DX));
		RowState0.DRs[1] = Utilities::Multiply(JointState.InvIs[1], FVec3::CrossProduct(JointState.Xs[1] - JointState.Ps[1], -DX));
	}

	FORCEINLINE void FJointSolver::ApplyRotationConstraint(
		const FReal Dt,
		const FJointSolverJointState& JointState,
		const FJointSolverConstraintRowData& RowData,
		FJointSolverConstraintRowState& RowState)
	{
		// Joint-space inverse mass
		FVec3 IA0 = FVec3(0);
		FVec3 IA1 = FVec3(0);
		FReal II0 = 0.0f;
		FReal II1 = 0.0f;
		if (JointState.InvMs[0] > 0)
		{
			IA0 = Utilities::Multiply(JointState.InvIs[0], RowState.Axis);
			II0 = FVec3::DotProduct(RowState.Axis, IA0);
		}
		if (JointState.InvMs[1] > 0)
		{
			IA1 = Utilities::Multiply(JointState.InvIs[1], RowState.Axis);
			II1 = FVec3::DotProduct(RowState.Axis, IA1);
		}
		const FReal II = (II0 + II1);

		if (RowData.bIsSoft)
		{
			// Damping angular velocity
			FReal AngVelDt = 0;
			if (RowData.Damping > 0.0f)
			{
				const FVec3 W0 = FRotation3::CalculateAngularVelocity(JointState.PrevQs[0], JointState.Qs[0], 1.0f);
				const FVec3 W1 = FRotation3::CalculateAngularVelocity(JointState.PrevQs[1], JointState.Qs[1], 1.0f);
				AngVelDt = FVec3::DotProduct(RowState.Axis, W0) - FVec3::DotProduct(RowState.Axis, W1);
			}

			const FReal SpringMassScale = (RowData.bIsAccelerationMode) ? 1.0f / II : 1.0f;
			const FReal S = SpringMassScale * RowData.Stiffness * Dt * Dt;
			const FReal D = SpringMassScale * RowData.Damping * Dt;
			const FReal Multiplier = (FReal)1 / ((S + D) * II + (FReal)1);
			const FReal DLambda = Multiplier * (S * RowState.Error - D * AngVelDt - RowState.Lambda);

			//const FVec3 DR0 = IA0 * DLambda;
			//const FVec3 DR1 = IA1 * -DLambda;
			RowState.DRs[0] = RowState.Axis * (DLambda * II0);
			RowState.DRs[1] = RowState.Axis * -(DLambda * II1);
			RowState.Lambda += DLambda;
		}
		else
		{
			RowState.DRs[0] = IA0 * (RowData.Stiffness * RowState.Error / II);
			RowState.DRs[1] = IA1 * -(RowData.Stiffness * RowState.Error / II);
		}
	}

}