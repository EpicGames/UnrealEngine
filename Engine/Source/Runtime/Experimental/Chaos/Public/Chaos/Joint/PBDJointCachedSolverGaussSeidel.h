// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "Chaos/DenseMatrix.h"
#include "Chaos/Evolution/SolverBody.h"
#include "Chaos/ParticleHandleFwd.h"
#include "Chaos/PBDJointConstraintTypes.h"
#include "Chaos/PBDJointConstraintUtilities.h"
#include "Chaos/Utilities.h"

namespace Chaos
{

/** Cached axis joint datas that will be used during the apply */
struct FAxisConstraintDatas
{
	FAxisConstraintDatas() {}
	FAxisConstraintDatas(const FAxisConstraintDatas&) = delete;
	FAxisConstraintDatas& operator=(const FAxisConstraintDatas&) = delete;
	FAxisConstraintDatas(FAxisConstraintDatas&&) = delete;
	FAxisConstraintDatas& operator=(FAxisConstraintDatas&&) = delete;

	/** Init the axis joint datas with stiffness / damping */
	void InitDatas(
		const int32 ConstraintIndex,
		const bool bHasSoftLimits,
		const FReal SoftStiffness,
		const FReal SoftDamping,
		const FReal HardStiffness,
		const bool bResetLambdas = true);

	/** Initialize the max force ( force is modified for acceleration mode and converted to positional impulse limit) */
	void SetMaxForce(
		const int32 ConstraintIndex,
		const FReal InMaxForce,
		const FReal Dt);

	/** Update the axis joint datas with axis, limits, arms... */
	void UpdateDatas(
		const int32 ConstraintIndex,
		const FVec3& DatasAxis,
		const FReal DatasCX,
		const FReal DatasRestitution,
		const bool bCheckLimit = true,
		const FVec3& DatasArm0 = FVec3::Zero(),
		const FVec3& DatasArm1 = FVec3::Zero(),
		const FReal DatasVX = 0.0);

	/** Update the mass dependent datas */
	void UpdateMass(
		const int32 ConstraintIndex,
		const FVec3& DatasIA0,
		const FVec3& DatasIA1,
		const FReal DatasIM,
		const FReal Dt,
		const bool bUsePositionBasedDrives);

	/** Apply the impulse limits to the impulse delta and net impulse */
	void ApplyMaxLambda(
		const int32 ConstraintIndex,
		FReal& DeltaLambda, 
		FReal& Lambda);

	/** Apply the impulse limits to the impulse delta and net impulse in SIMD */
	void ApplyMaxLambdaSIMD(
		VectorRegister4Float& DeltaLambda,
		VectorRegister4Float& Lambda);
	
	

	struct alignas(16) FDataSimd
	{
		FDataSimd() 
			: ConstraintSoftStiffness(VectorZeroFloat())
			, ConstraintSoftDamping(VectorZeroFloat())
			, ConstraintLimits(VectorZeroFloat())
		{}

		VectorRegister4Float ConstraintHardStiffness;
		VectorRegister4Float ConstraintSoftStiffness;
		VectorRegister4Float ConstraintSoftDamping;
		VectorRegister4Float ConstraintArms[2];
		VectorRegister4Float ConstraintAxis[3];
		VectorRegister4Float ConstraintLimits;
		VectorRegister4Float ConstraintSoftIM;
		VectorRegister4Float ConstraintHardIM;
		VectorRegister4Float ConstraintDRAxis[3][2];
		VectorRegister4Float ConstraintCX;
		VectorRegister4Float ConstraintLambda;
	};

	struct FData
	{
		FVec3 ConstraintHardStiffness;
		FVec3 ConstraintSoftStiffness;
		FVec3 ConstraintSoftDamping;
		FVec3 ConstraintArms[3][2];
		FVec3 ConstraintAxis[3];
		FVec3 ConstraintLimits;
		FVec3 ConstraintSoftIM;
		FVec3 ConstraintHardIM;
		FVec3 ConstraintDRAxis[3][2];
		FVec3 ConstraintCX;
		FVec3 ConstraintLambda;
	};

	union
	{
		FDataSimd Simd;
		FData Data;
	};

	FVec3 ConstraintMaxLambda;
	FVec3 SettingsSoftDamping;
	FVec3 SettingsSoftStiffness;
	FVec3 ConstraintVX;

	FVec3 ConstraintRestitution;
	FVec3 ConstraintLambdaVelocity;
	
	uint8 bValidDatas0 : 1;
	uint8 bValidDatas1 : 1;
	uint8 bValidDatas2 : 1;
	uint8 bLimitsCheck0 : 1;
	uint8 bLimitsCheck1 : 1;
	uint8 bLimitsCheck2 : 1;
	uint8 bSoftLimit0 : 1;
	uint8 bSoftLimit1 : 1;
	uint8 bSoftLimit2 : 1;
	uint8 bAccelerationMode : 1;
	uint8 bUseSimd : 1;
	uint8 MotionType0 : 2;
	uint8 MotionType1 : 2;
	uint8 MotionType2 : 2;

	inline bool GetValidDatas(int32 Index) const 
	{
		switch (Index)
		{
		case 0: return bValidDatas0;
		case 1: return bValidDatas1;
		case 2: return bValidDatas2;
		default: check(false);  return false;
		}
	}

	inline bool GetLimitsCheck(int32 Index) const
	{
		switch (Index)
		{
		case 0: return bLimitsCheck0;
		case 1: return bLimitsCheck1;
		case 2: return bLimitsCheck2;
		default: check(false);  return false;
		}
	}

	inline bool GetSoftLimit(int32 Index) const
	{
		switch (Index)
		{
		case 0: return bSoftLimit0;
		case 1: return bSoftLimit1;
		case 2: return bSoftLimit2;
		default: check(false);  return false;
		}
	}

	inline EJointMotionType GetMotionType(int32 Index) const
	{
		switch (Index)
		{
		case 0: return EJointMotionType(MotionType0);
		case 1: return EJointMotionType(MotionType1);
		case 2: return EJointMotionType(MotionType2);
		default: check(false);  return EJointMotionType();
		}
	}

	inline void SetValidDatas(int32 Index, bool bValue)
	{
		switch (Index)
		{
		case 0: bValidDatas0 = bValue; break;
		case 1: bValidDatas1 = bValue; break;
		case 2: bValidDatas2 = bValue; break;
		default: check(false);
		}
	}

	inline void SetLimitsCheck(int32 Index, bool bValue)
	{
		switch (Index)
		{
		case 0: bLimitsCheck0 = bValue; break;
		case 1: bLimitsCheck1 = bValue; break;
		case 2: bLimitsCheck2 = bValue; break;
		default: check(false);
		}
	}

	inline void SetSoftLimit(int32 Index, bool bValue)
	{
		switch (Index)
		{
		case 0: bSoftLimit0 = bValue; break;
		case 1: bSoftLimit1 = bValue; break;
		case 2: bSoftLimit2 = bValue; break;
		default: check(false);
		}
	}

    inline void SetMotionType(int32 Index, EJointMotionType Value)
	{
		check(int32(Value) <= 3); // Stored only on 2 bits
		switch (Index)
		{
		case 0: MotionType0 = uint8(Value); break;
		case 1: MotionType1 = uint8(Value); break;
		case 2: MotionType2 = uint8(Value); break;
		default: check(false);
		}
	}

};
	
	/**
	 * Calculate new positions and rotations for a pair of bodies connected by a joint.
	 *
	 * This solver treats of the 6 possible constraints (up to 3 linear and 3 angular)
	 * individually and resolves them in sequence.
	 *
	 * \see FJointSolverCholesky
	 */
	class FPBDJointCachedSolver
	{
	public:
		static const int32 MaxConstrainedBodies = 2;

		// Access to the SolverBodies that the constraint is operating on.
		// \note SolverBodies are transient and exist only as long as we are in the Island's constraint solver loop
		inline FConstraintSolverBody& Body(int32 BodyIndex)
		{
			check((BodyIndex >= 0) && (BodyIndex < 2));
			check(SolverBodies[BodyIndex].IsValid());

			return SolverBodies[BodyIndex];
		}

		inline const FConstraintSolverBody& Body(int32 BodyIndex) const
		{
			check((BodyIndex >= 0) && (BodyIndex < 2));
			check(SolverBodies[BodyIndex].IsValid());

			return SolverBodies[BodyIndex];
		}

		inline FConstraintSolverBody& Body0()
		{
			return SolverBodies[0];
		}

		inline const FConstraintSolverBody& Body0() const
		{
			return SolverBodies[0];
		}

		inline FConstraintSolverBody& Body1()
		{
			return SolverBodies[1];
		}

		inline const FConstraintSolverBody& Body1() const
		{
			return SolverBodies[1];
		}

		inline const FVec3 X(int BodyIndex) const
		{
			return Body(BodyIndex).X();
		}

		inline const FRotation3 R(int BodyIndex) const
		{
			return Body(BodyIndex).R();
		}

		inline const FVec3 P(int BodyIndex) const
		{
			// NOTE: Joints always use the latest post-correction position and rotation. This makes the joint error calculations non-linear and more robust against explosion
			// but adds a cost because we need to apply the latest correction each time we request the latest transform
			return Body(BodyIndex).CorrectedP();
		}

		inline const FRotation3 Q(int BodyIndex) const
		{
			// NOTE: Joints always use the latest post-correction position and rotation. This makes the joint error calculations non-linear and more robust against explosion
			// but adds a cost because we need to apply the latest correction each time we request the latest transform
			return Body(BodyIndex).CorrectedQ();
		}

		inline const FVec3 V(int BodyIndex) const
		{
			return Body(BodyIndex).V();
		}

		inline const FVec3 W(int BodyIndex) const
		{
			return Body(BodyIndex).W();
		}

		inline FReal InvM(int32 BodyIndex) const
		{
			return InvMs[BodyIndex];
		}

		inline FMatrix33 InvI(int32 BodyIndex) const
		{
			return InvIs[BodyIndex];
		}

		inline bool IsDynamic(int32 BodyIndex) const
		{
			return (InvM(BodyIndex) > 0);
		}

		// NOTE: This is a positional impulse
		inline FVec3 GetNetLinearImpulse() const
		{
			FVec3 Impulse = FVec3(0);
			if (PositionConstraints.bUseSimd || PositionDrives.bUseSimd)
			{
				VectorRegister4Float ImpulseSimd = VectorZeroFloat();
				if (PositionConstraints.bUseSimd)
				{
					VectorRegister4Float ConstraintLambdas[3];
					ConstraintLambdas[0] = VectorReplicate(PositionConstraints.Simd.ConstraintLambda, 0);
					ConstraintLambdas[1] = VectorReplicate(PositionConstraints.Simd.ConstraintLambda, 1);
					ConstraintLambdas[2] = VectorReplicate(PositionConstraints.Simd.ConstraintLambda, 2);
					for (int32 Axis = 0; Axis < 3; ++Axis)
					{
						ImpulseSimd = VectorMultiplyAdd(ConstraintLambdas[Axis], PositionConstraints.Simd.ConstraintAxis[Axis], ImpulseSimd);
					}
				}
				if (PositionDrives.bUseSimd)
				{
					VectorRegister4Float ConstraintLambdas[3];
					ConstraintLambdas[0] = VectorReplicate(PositionDrives.Simd.ConstraintLambda, 0);
					ConstraintLambdas[1] = VectorReplicate(PositionDrives.Simd.ConstraintLambda, 1);
					ConstraintLambdas[2] = VectorReplicate(PositionDrives.Simd.ConstraintLambda, 2);
					for (int32 Axis = 0; Axis < 3; ++Axis)
					{
						ImpulseSimd = VectorMultiplyAdd(ConstraintLambdas[Axis], PositionDrives.Simd.ConstraintAxis[Axis], ImpulseSimd);
					}
				}
				FVec3f Impulsef;
				VectorStoreFloat3(ImpulseSimd, &Impulsef[0]);
				Impulse = FVec3(Impulsef);
			}

			for (int32 Axis = 0; Axis < 3; ++Axis)
			{
				if (!PositionConstraints.bUseSimd && PositionConstraints.GetValidDatas(Axis))
				{
					Impulse += PositionConstraints.Data.ConstraintLambda[Axis] * PositionConstraints.Data.ConstraintAxis[Axis];
				}
				if (!PositionDrives.bUseSimd && PositionDrives.GetValidDatas(Axis))
				{
					Impulse += PositionDrives.Data.ConstraintLambda[Axis] * PositionDrives.Data.ConstraintAxis[Axis];
				}
			}
			return Impulse;
		}

		// NOTE: This is a positional impulse
		inline FVec3 GetNetAngularImpulse() const
		{
			FVec3 Impulse = FVec3(0);
			if (RotationConstraints.bUseSimd || RotationDrives.bUseSimd)
			{
				VectorRegister4Float ImpulseSimd = VectorZeroFloat();
				if (RotationConstraints.bUseSimd)
				{
					VectorRegister4Float ConstraintLambdas[3];
					ConstraintLambdas[0] = VectorReplicate(RotationConstraints.Simd.ConstraintLambda, 0);
					ConstraintLambdas[1] = VectorReplicate(RotationConstraints.Simd.ConstraintLambda, 1);
					ConstraintLambdas[2] = VectorReplicate(RotationConstraints.Simd.ConstraintLambda, 2);
					for (int32 Axis = 0; Axis < 3; ++Axis)
					{
						ImpulseSimd = VectorMultiplyAdd(ConstraintLambdas[Axis], RotationConstraints.Simd.ConstraintAxis[Axis], ImpulseSimd);
					}
				}
				if (RotationDrives.bUseSimd)
				{
					VectorRegister4Float ConstraintLambdas[3];
					ConstraintLambdas[0] = VectorReplicate(RotationDrives.Simd.ConstraintLambda, 0);
					ConstraintLambdas[1] = VectorReplicate(RotationDrives.Simd.ConstraintLambda, 1);
					ConstraintLambdas[2] = VectorReplicate(RotationDrives.Simd.ConstraintLambda, 2);
					for (int32 Axis = 0; Axis < 3; ++Axis)
					{
						ImpulseSimd = VectorMultiplyAdd(ConstraintLambdas[Axis], RotationDrives.Simd.ConstraintAxis[Axis], ImpulseSimd);
					}
				}
				FVec3f Impulsef;
				VectorStoreFloat3(ImpulseSimd, &Impulsef[0]);
				Impulse = FVec3(Impulsef);
			}

			for (int32 Axis = 0; Axis < 3; ++Axis)
			{
				if (!RotationConstraints.bUseSimd && RotationConstraints.GetValidDatas(Axis))
				{
					Impulse += RotationConstraints.Data.ConstraintLambda[Axis] * RotationConstraints.Data.ConstraintAxis[Axis];
				}
				if (!RotationDrives.bUseSimd && RotationDrives.GetValidDatas(Axis))
				{
					Impulse += RotationDrives.Data.ConstraintLambda[Axis] * RotationDrives.Data.ConstraintAxis[Axis];
				}
			}
			return Impulse;
		}

		inline FReal GetLinearViolationSq() const
		{
			return -1.f;
		}

		inline FReal GetAngularViolation() const
		{
			return -1.f;
		}

		inline int32 GetNumActiveConstraints() const
		{
			// We use -1 as unitialized, but that should not be exposed outside the solver
			return FMath::Max(NumActiveConstraints, 0);
		}

		inline bool GetIsActive() const
		{
			return bIsActive;
		}
		
		FPBDJointCachedSolver()
		{
		}

		void SetSolverBodies(FSolverBody* SolverBody0, FSolverBody* SolverBody1)
		{
			SolverBodies[0] = *SolverBody0;
			SolverBodies[1] = *SolverBody1;
		}

		// Called once per frame to initialize the joint solver from the joint settings
		void Init(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings,
			const FRigidTransform3& XL0,
			const FRigidTransform3& XL1);

		void InitProjection(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		void Deinit();

		// @todo(chaos): this doesn't do much now we have SolverBodies - remove it
		void Update(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		// Run the position solve for the constraints
		void ApplyConstraints(
			const FReal Dt,
			const FReal InSolverStiffness,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		// Run the velocity solve for the constraints
		void ApplyVelocityConstraints(
			const FReal Dt,
			const FReal InSolverStiffness,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		// Apply projection (a position solve where the parent has infinite mass) for the constraints
		// @todo(chaos): this can be build into the Apply phase when the whole solver is PBD
		void ApplyProjections(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings,
			const bool bLastIteration);

		void ApplyPositionProjection(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		void ApplyAxisPositionProjection(
			const FReal LinearProjection,
			const int32 ConstraintIndex);

		void ApplyPositionProjectionSimd(
			const FReal LinearProjection);

		void ApplyRotationProjection(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		void ApplyAxisRotationProjection(
			const FReal AngularProjection,
			const bool bLinearLocked,
			const int32 ConstraintIndex);

		void ApplyRotationProjectionSimd(
			const FRealSingle AngularProjection,
			const bool bLinearLocked);

		void ApplyTeleports(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		void ApplyPositionTeleport(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		void ApplyAxisPositionTeleport(
			const FReal TeleportDistance,
			const int32 ConstraintIndex);

		void ApplyPositionTeleportSimd(
			const FRealSingle TeleportDistance);

		void ApplyRotationTeleport(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		void SetShockPropagationScales(
			const FReal InvMScale0,
			const FReal InvMScale1,
			const FReal Dt);

		void SetIsBroken(const bool bInIsBroken)
		{
			bIsBroken = bInIsBroken;
		}

		bool IsBroken() const
		{
			return bIsBroken;
		}

		void SetIsViolating(const bool bInIsViolating)
		{
			bIsViolating = bInIsViolating;
		}

		bool IsViolating() const
		{
			return bIsViolating;
		}

		bool RequiresSolve() const
		{
			return !IsBroken() && (IsDynamic(0) || IsDynamic(1));
		}

	private:

		// Initialize state dependent on particle positions
		void InitDerivedState();

		// Update state dependent on particle positions (after moving one or both of them)
		void UpdateDerivedState(const int32 BodyIndex);
		void UpdateDerivedState();

		void UpdateMass0(const FReal& InInvM, const FVec3& InInvIL);
		void UpdateMass1(const FReal& InInvM, const FVec3& InInvIL);

		// Check to see if this constraint still needs further solved
		// @todo(chaos): the term "active" is used inconsistently with the meaning elsewhere. Active should
		// mean "contributed impusles". Should use "solved" rather than "active"
		bool UpdateIsActive();

		/** Common function to apply the lagrange multiplier update */
		
		void ApplyPositionDelta(
			const int32 BodyIndex,
			const FVec3& DP);

		void ApplyRotationDelta(
			const int32 BodyIndex,
			const FVec3& DR);

		/** Init Position constraints */
		
		void InitPositionConstraints(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings,
			const bool bResetLambdas);

		void InitPositionDatasMass(
			FAxisConstraintDatas& PositionDatas,
			const int32 ConstraintIndex,
			const FReal Dt);
		
		void InitPositionConstraintDatas(
			const int32 ConstraintIndex,
			const FVec3& ConstraintAxis,
			const FReal& ConstraintDelta,
			const FReal ConstraintRestitution,
			const FReal Dt,
			const FReal ConstraintLimit,
			const EJointMotionType JointType,
			const FVec3& ConstraintArm0,
			const FVec3& ConstraintArm1);

		void InitLockedPositionConstraint(
			const FPBDJointSettings& JointSettings,
			const FReal Dt,
			const TVec3<EJointMotionType>& LinearMotion);

		void InitLockedPositionConstraintSimd(
			const FPBDJointSettings& JointSettings,
			const FReal Dt,
			const TVec3<EJointMotionType>& LinearMotion);

		void InitSphericalPositionConstraint(
			const FPBDJointSettings& JointSettings,
			const FReal Dt);
		
		void InitCylindricalPositionConstraint(
			const FPBDJointSettings& JointSettings,
			const FReal Dt,
			const int32 AxisIndex);

		void InitPlanarPositionConstraint(
			const FPBDJointSettings& JointSettings,
			const FReal Dt,
			const int32 AxisIndex);
		
		/** Apply Position constraints */

		void ApplyPositionConstraints(
			const FReal Dt);
		
		void ApplyAxisPositionConstraint(
			const int32 ConstraintIndex,
			const FReal Dt);

		void ApplyPositionConstraintsSimd(
			const FReal Dt);
		
		void SolvePositionConstraintDelta(
			const int32 ConstraintIndex, 
			const FReal DeltaLambda,
			const FAxisConstraintDatas& ConstraintDatas);

		void SolvePositionConstraintHard(
			const int32 ConstraintIndex,
			const FReal DeltaConstraint);

		void SolvePositionConstraintSoft(
			const int32 ConstraintIndex,
			const FReal DeltaConstraint,
			const FReal Dt,
			const FReal TargetVel);
		
		/** Apply Linear Velocity constraints */
		
		void ApplyLinearVelocityConstraints();

		void ApplyAxisVelocityConstraint(
			const int32 ConstraintIndex);

		void ApplyVelocityConstraintSimd();

		void SolveLinearVelocityConstraint(
			const int32 ConstraintIndex,
			const FReal TargetVel);
		
		/** Init Rotation constraints */
		
		void InitRotationConstraints(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings,
			const bool bResetLambdas);

		void InitRotationConstraintsSimd(
			const FPBDJointSettings& JointSettings,
			const FRealSingle Dt);

		void CorrectAxisAngleConstraint(
			const FPBDJointSettings& JointSettings,
			const int32 ConstraintIndex,
			FVec3& ConstraintAxis,
			FReal& ConstraintAngle) const;

		void InitRotationConstraintDatas(
			const FPBDJointSettings& JointSettings,
			const int32 ConstraintIndex,
			const FVec3& ConstraintAxis,
			const FReal ConstraintAngle,
			const FReal ConstraintRestitution,
			const FReal Dt,
			const bool bCheckLimit);

		void InitRotationDatasMass(
			FAxisConstraintDatas& RotationDatas, 
			const int32 ConstraintIndex,
			const FReal Dt);
		
		void InitTwistConstraint(
			const FPBDJointSettings& JointSettings,
			const FReal Dt);
		
		void InitConeConstraint(
			const FPBDJointSettings& JointSettings,
			const FReal Dt);

		void InitPyramidSwingConstraint(
		   const FPBDJointSettings& JointSettings,
		   const FReal Dt,
		   const bool bApplySwing1,
		   const bool bApplySwing2);

		// One Swing axis is free, and the other locked. This applies the lock: Body1 Twist axis is confined to a plane.
		void InitSingleLockedSwingConstraint(
			const FPBDJointSettings& JointSettings,
			const FReal Dt,
			const EJointAngularConstraintIndex SwingConstraintIndex);

		// One Swing axis is free, and the other limited. This applies the limit: Body1 Twist axis is confined to space between two cones.
		void InitDualConeSwingConstraint(
			const FPBDJointSettings& JointSettings,
			const FReal Dt,
			const EJointAngularConstraintIndex SwingConstraintIndex);

		// One swing axis is locked, the other limited or locked. This applies the Limited axis (ApplyDualConeSwingConstraint is used for the locked axis).
		void InitSwingConstraint(
			const FPBDJointSettings& JointSettings,
			const FPBDJointSolverSettings& SolverSettings,
			const FReal Dt,
			const EJointAngularConstraintIndex SwingConstraintIndex);

		void InitLockedRotationConstraints(
			const FPBDJointSettings& JointSettings,
			const FReal Dt,
			const bool bApplyTwist,
			const bool bApplySwing1,
			const bool bApplySwing2);

		/** Apply Rotation constraints */

		void ApplyRotationConstraints(
			const FReal Dt);
		
		void ApplyRotationConstraint(
			const int32 ConstraintIndex,
			const FReal Dt);

		void ApplyRotationSoftConstraintsSimd(
			const FReal Dt);

		void SolveRotationConstraintDelta(
			const int32 ConstraintIndex, 
			const FReal DeltaLambda,
			const bool bIsSoftConstraint,
			const FAxisConstraintDatas& ConstraintDatas);

		void SolveRotationConstraintHard(
			const int32 ConstraintIndex,
			const FReal DeltaConstraint);

		void SolveRotationConstraintSoft(
			const int32 ConstraintIndex,
			const FReal DeltaConstraint,
			const FReal Dt,
			const FReal TargetVel);

		/** Apply Angular velocity  constraint */

		void ApplyAngularVelocityConstraints();

		void SolveAngularVelocityConstraint(
			const int32 ConstraintIndex,
			const FReal TargetVel);

		void ApplyAngularVelocityConstraint(
			const int32 ConstraintIndex);

		void ApplyAngularVelocityConstraintSimd();

		/** Init Position Drives */
		
		void InitPositionDrives(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		void InitAxisPositionDrive(
            const int32 ConstraintIndex,
            const FVec3& ConstraintAxis,
            const FVec3& DeltaPosition,
            const FVec3& DeltaVelocity,
            const FReal Dt);

		/** Apply Position Drives */
		
		void ApplyPositionDrives(
			const FReal Dt);
		
		void ApplyAxisPositionDrive(
			const int32 ConstraintIndex,
			const FReal Dt);

		void ApplyPositionVelocityDrives(
			const FReal Dt);

		void ApplyAxisPositionVelocityDrive(
			const int32 ConstraintIndex,
			const FReal Dt);

		/** Init Rotation Drives */

		void InitRotationDrives(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		void InitRotationConstraintDrive(
			const int32 ConstraintIndex,
			const FVec3& ConstraintAxis,
			const FReal Dt,
			const FReal DeltaAngle);

		void InitRotationConstraintDriveSimd(
			FVec3 ConstraintAxes[3],
			const FRealSingle Dt,
			const FVec3 DeltaAngles);
		
		void InitSwingTwistDrives(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings,
			const bool bTwistDriveEnabled,
			const bool bSwing1DriveEnabled,
			const bool bSwing2DriveEnabled);
		
		void InitSLerpDrive(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		/** Apply Rotation Drives */
		
		void ApplyRotationDrives(
			const FReal Dt);
		
		void ApplyAxisRotationDrive(
			const int32 ConstraintIndex,
			const FReal Dt);

		void ApplyRotationDrivesSimd(
			const FReal Dt);

		void ApplyRotationVelocityDrives(
			const FReal Dt);

		void ApplyAxisRotationVelocityDrive(
			const int32 ConstraintIndex,
			const FReal Dt);

		void SetInitConstraintVelocity(
			const FVec3& ConstraintArm0,
			const FVec3& ConstraintArm1);

		
		// The cached body state on which the joint operates
		FConstraintSolverBody SolverBodies[MaxConstrainedBodies];

		// Local-space constraint settings
		FRigidTransform3 LocalConnectorXs[MaxConstrainedBodies];	// Local(CoM)-space joint connector transforms

		// World-space constraint settings
		FVec3 ConnectorXs[MaxConstrainedBodies];			// World-space joint connector positions
		FRotation3 ConnectorRs[MaxConstrainedBodies];		// World-space joint connector rotations
		FVec3 ConnectorWDts[MaxConstrainedBodies];			// World-space joint connector angular velocities * dt
		VectorRegister4Float ConnectorWDtsSimd[MaxConstrainedBodies];			// World-space joint connector angular velocities * dt

		// XPBD Initial iteration world-space body state
		FVec3 InitConnectorXs[MaxConstrainedBodies];		// World-space joint connector positions
		FRotation3 InitConnectorRs[MaxConstrainedBodies];	// World-space joint connector rotations
		FVec3 InitConstraintVelocity;						// Initial relative velocity at the constrained position

		// Inverse Mass and Inertia
		FReal InvMs[MaxConstrainedBodies];
		FMatrix33 InvIs[MaxConstrainedBodies];				// World-space
		
		// Solver stiffness - increased over iterations for stability
		// \todo(chaos): remove Stiffness from SolverSettings (because it is not a solver constant)
		FReal SolverStiffness;

		// Tolerances below which we stop solving
		FReal PositionTolerance;							// Distance error below which we consider a constraint or drive solved
		FReal AngleTolerance;								// Angle error below which we consider a constraint or drive solved

		// Tracking whether the solver is resolved
		FVec3 LastDPs[MaxConstrainedBodies];				// Positions at the beginning of the iteration
		FVec3 LastDQs[MaxConstrainedBodies];				// Rotations at the beginning of the iteration
		FVec3 CurrentPs[MaxConstrainedBodies];				// Positions at the beginning of the iteration
		FRotation3 CurrentQs[MaxConstrainedBodies];			// Rotations at the beginning of the iteration
		FVec3 InitConstraintAxisLinearVelocities;			// Linear velocities along the constraint axes at the beginning of the frame, used by restitution
		FVec3 InitConstraintAxisAngularVelocities;			// Angular velocities along the constraint axes at the beginning of the frame, used by restitution
		int32 NumActiveConstraints;							// The number of active constraints and drives in the last iteration (-1 initial value)
		bool bIsActive;										// Whether any constraints actually moved any bodies in last iteration
		bool bUsePositionBasedDrives;						// Whether to apply velocity drive in the PBD step of the VBD step
		
		FAxisConstraintDatas PositionConstraints;
		FAxisConstraintDatas RotationConstraints;

		FAxisConstraintDatas PositionDrives;
		FAxisConstraintDatas RotationDrives;

		bool bIsBroken;
		bool bIsViolating;
		bool bUseSimd;

		// dummy indices
		static constexpr int32 PointPositionConstraintIndex = 0;
		static constexpr int32 SphericalPositionConstraintIndex = 0;
		static constexpr int32 PlanarPositionConstraintIndex = 0;

	private :
		/** Compute the body and connector state
		 * @param BodyIndex Index of the body (0,1) on which the state will be updated
		 */
		void ComputeBodyState(const int32 BodyIndex);
	};

}