// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/Evolution/SolverConstraintContainer.h"
#include "Chaos/Joint/PBDJointSolverGaussSeidel.h"
#include "Chaos/Joint/PBDJointCachedSolverGaussSeidel.h"
#include "Chaos/PBDJointConstraints.h"
#include "Chaos/PBDJointConstraintTypes.h"

namespace Chaos
{
	namespace Private
	{
		/**
		 * Runs the solvers for a set of constraints belonging to a JointConstraints container.
		 * 
		 * For the main scene, each IslandGroup owns two TPBDJointContainerSolvers (one for linear and one for nonlinear) and the list of constraints to be solved
		 * and the order in which they are solved is determined by the constraint graph. The two TPBDJointContainerSolvers are grouped under struct FPBDJointCombinedConstraints.
		 * 
		 * For RBAN, there is one FPBDJointCombinedConstraints with two TPBDJointContainerSolvers (linear & nonlinear) that solves all joints in the simulation in the order that
		 * they occur in the container.
		*/
		template <typename JointSolverType>
		class TPBDJointContainerSolver : public FConstraintContainerSolver
		{
		public:
			TPBDJointContainerSolver(FPBDJointConstraints& InConstraintContainer, const int32 InPriority);
			~TPBDJointContainerSolver();

			// FConstraintContainerSolver impl
			virtual int32 GetNumConstraints() const override final { return ContainerIndices.Num();}
			virtual void AddConstraints() override final;
			virtual void AddConstraints(const TArrayView<Private::FPBDIslandConstraint*>& IslandConstraints) override final;
			virtual void AddBodies(FSolverBodyContainer& SolverBodyContainer) override final;
			virtual void GatherInput(const FReal Dt) override final;
			virtual void GatherInput(const FReal Dt, const int32 BeginIndex, const int32 EndIndex) override final;
			virtual void ScatterOutput(const FReal Dt) override final;
			virtual void ScatterOutput(const FReal Dt, const int32 BeginIndex, const int32 EndIndex) override final;
			virtual void ApplyPositionConstraints(const FReal Dt, const int32 It, const int32 NumIts) override final;
			virtual void ApplyPositionConstraints(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex) override final;
			virtual void ApplyVelocityConstraints(const FReal Dt, const int32 It, const int32 NumIts) override final;
			virtual void ApplyVelocityConstraints(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex) override final;
			virtual void PreApplyProjectionConstraints(const FReal Dt) override final;
			virtual void PreApplyProjectionConstraints(const FReal Dt, const int32 BeginIndex, const int32 EndIndex) override final;
			virtual void ApplyProjectionConstraints(const FReal Dt, const int32 It, const int32 NumIts) override final;
			virtual void ApplyProjectionConstraints(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex) override final;

			FPBDJointConstraints& GetContainer() const { return ConstraintContainer; }
			const FPBDJointSolverSettings& GetSettings() const { return ConstraintContainer.GetSettings(); }
			const FPBDJointSettings& GetConstraintSettings(const int32 InConstraintIndex) const { return ConstraintContainer.GetConstraintSettings(ContainerIndices[InConstraintIndex]);}
			int32 GetContainerConstraintIndex(const int32 InConstraintIndex) const { return ContainerIndices[InConstraintIndex];}
		private:
			virtual void ResetImpl(const int32 InMaxCollisions) override final;
			virtual bool SupportsPartitioning() const override final { return true; }
			virtual bool SortConstraintsByPartitionIndex(const TArray<int32>& PartitionIndices, const TArray<int32>& PrefixSum) override final;
			virtual void GetAllConstrainedParticles(TArray<const FGeometryParticleHandle*>& ParticleHandles) const override final;

			bool UseLinearSolver() const;
			void AddConstraint(const int32 InContainerConstraintIndex);
			void ResizeSolverArrays();
			void PreApplyLinearProjectionConstraints(const FReal Dt, const int32 BeginIndex, const int32 EndIndex);
			void PreApplyNonLinearProjectionConstraints(const FReal Dt, const int32 BeginIndex, const int32 EndIndex);
			void ApplyLinearProjectionConstraints(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex);
			void ApplyNonLinearProjectionConstraints(const FReal Dt, const int32 It, const int32 NumIts, const int32 BeginIndex, const int32 EndIndex);

			FPBDJointConstraints& ConstraintContainer;

			TArray<JointSolverType> ConstraintSolvers;

			// Index remapping from respective internal index of each array to index in the joint container [0,NumJointsInWorld)
			TArray<int32> ContainerIndices;
		};
		
	}
}