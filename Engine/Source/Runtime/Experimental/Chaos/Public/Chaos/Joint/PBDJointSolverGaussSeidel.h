// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "Chaos/DenseMatrix.h"
#include "Chaos/Evolution/SolverBody.h"
#include "Chaos/ParticleHandleFwd.h"
#include "Chaos/PBDJointConstraintTypes.h"
#include "Chaos/PBDJointConstraintUtilities.h"
#include "Chaos/Utilities.h"

namespace Chaos
{
	class FPBDJointSolver;

	UE_DEPRECATED(4.27, "Use FPBDJointSolver")
	typedef FPBDJointSolver FJointSolverGaussSeidel;

	/**
	 * Calculate new positions and rotations for a pair of bodies connected by a joint.
	 *
	 * This solver treats of the 6 possible constraints (up to 3 linear and 3 angular)
	 * individually and resolves them in sequence.
	 *
	 * \see FJointSolverCholesky
	 */
	class FPBDJointSolver
	{
	public:
		static const int32 MaxConstrainedBodies = 2;

		// Access to the SolverBodies that the constraint is operating on.
		// \note SolverBodies are transient and exist only as long as we are in the Island's constraint solver loop
		inline FConstraintSolverBody& Body(int32 BodyIndex)
		{
			check((BodyIndex >= 0) && (BodyIndex < 2));
			check(SolverBodies[BodyIndex].IsValid());

			return SolverBodies[BodyIndex];
		}

		inline const FConstraintSolverBody& Body(int32 BodyIndex) const
		{
			check((BodyIndex >= 0) && (BodyIndex < 2));
			check(SolverBodies[BodyIndex].IsValid());

			return SolverBodies[BodyIndex];
		}

		inline FConstraintSolverBody& Body0()
		{
			return SolverBodies[0];
		}

		inline const FConstraintSolverBody& Body0() const
		{
			return SolverBodies[0];
		}

		inline FConstraintSolverBody& Body1()
		{
			return SolverBodies[1];
		}

		inline const FConstraintSolverBody& Body1() const
		{
			return SolverBodies[1];
		}

		inline const FVec3& X(int BodyIndex) const
		{
			return Body(BodyIndex).X();
		}

		inline const FRotation3& R(int BodyIndex) const
		{
			return Body(BodyIndex).R();
		}

		inline const FVec3 P(int BodyIndex) const
		{
			// NOTE: Joints always use the latest post-correction position and rotation. This makes the joint error calculations non-linear and more robust against explosion
			// but adds a cost because we need to apply the latest correction each time we request the latest transform
			return Body(BodyIndex).CorrectedP();
		}

		inline const FRotation3 Q(int BodyIndex) const
		{
			// NOTE: Joints always use the latest post-correction position and rotation. This makes the joint error calculations non-linear and more robust against explosion
			// but adds a cost because we need to apply the latest correction each time we request the latest transform
			return Body(BodyIndex).CorrectedQ();
		}

		inline FVec3 V(int BodyIndex) const
		{
			return Body(BodyIndex).V();
		}

		inline FVec3 W(int BodyIndex) const
		{
			return Body(BodyIndex).W();
		}

		inline FReal InvM(int32 BodyIndex) const
		{
			return InvMs[BodyIndex];
		}

		inline FMatrix33 InvI(int32 BodyIndex) const
		{
			return InvIs[BodyIndex];
		}

		inline bool IsDynamic(int32 BodyIndex) const
		{
			return (InvM(BodyIndex) > 0);
		}

		inline const FVec3& GetNetLinearImpulse() const
		{
			return NetLinearImpulse;
		}

		inline const FVec3& GetNetAngularImpulse() const
		{
			return NetAngularImpulse;
		}

		inline FReal GetLinearViolationSq() const
		{
			return LinearViolationSq;
		}

		inline FReal GetAngularViolation() const
		{
			return AngularViolation;
		}

		FPBDJointSolver()
		{
		}

		void SetSolverBodies(FSolverBody* SolverBody0, FSolverBody* SolverBody1)
		{
			SolverBodies[0] = *SolverBody0;
			SolverBodies[1] = *SolverBody1;
		}

		// Called once per frame to initialze the joint solver from the joint settings
		void Init(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings,
			const FRigidTransform3& XL0,
			const FRigidTransform3& XL1);

		void Deinit();

		// @todo(chaos): this doesn't do much now we have SolverBodies - remove it
		void Update(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		void UpdateMasses(
			const FReal InvMassScale0,
			const FReal InvMassScale1);

		// Run the position solve for the constraints
		void ApplyConstraints(
			const FReal Dt,
			const FReal InSolverStiffness,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		// Run the velocity solve for the constraints
		void ApplyVelocityConstraints(
			const FReal Dt,
			const FReal InSolverStiffness,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		// Apply projection (a position solve where the parent has infinite mass) for the constraints
		// @todo(chaos): this can be build into the Apply phase when the whole solver is PBD
		void ApplyProjections(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings,
			const bool bLastIteration);

		void SetShockPropagationScales(
			const FReal InvMScale0, 
			const FReal InvMScale1,
			const FReal Dt);

		void SetIsBroken(const bool bInIsBroken)
		{
			bIsBroken = bInIsBroken;
		}

		bool IsBroken() const
		{
			return bIsBroken;
		}

		void SetIsViolating(const bool bInIsViolating)
		{
			bIsViolating = bInIsViolating;
		}

		bool IsViolating() const
		{
			return bIsViolating;
		}

		bool RequiresSolve() const
		{
			return !IsBroken() && (IsDynamic(0) || IsDynamic(1));
		}

	private:

		// Initialize state dependent on particle positions
		void InitDerivedState();

		// Update state dependent on particle positions (after moving one or both of them)
		void UpdateDerivedState(const int32 BodyIndex);
		void UpdateDerivedState();

		void UpdateMass0();
		void UpdateMass1();

		void ApplyPositionConstraints(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		void ApplyRotationConstraints(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		void ApplyPositionDrives(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		void ApplyRotationDrives(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		void ApplyLinearVelocityConstraints(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		void ApplyAngularVelocityConstraints(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		void ApplyPositionDelta(
			const int32 BodyIndex,
			const FVec3& DP);

		void ApplyPositionDelta(
			const FVec3& DP0,
			const FVec3& DP1);

		void ApplyRotationDelta(
			const int32 BodyIndex,
			const FVec3& DR);

		void ApplyRotationDelta(
			const FVec3& DR0,
			const FVec3& DR1);

		void ApplyDelta(
			const int32 BodyIndex,
			const FVec3& DP,
			const FVec3& DR1);

		void ApplyVelocityDelta(
			const int32 BodyIndex,
			const FVec3& DV,
			const FVec3& DW);

		void ApplyVelocityDelta(
			const FVec3& DV0,
			const FVec3& DW0,
			const FVec3& DV1,
			const FVec3& DW1);

		void ApplyAngularVelocityDelta(
			const FVec3& DW0,
			const FVec3& DW1);

		void ApplyPositionConstraint(
			const FReal JointStiffness,
			const FVec3& Axis,
			const FReal Delta,
			const FVec3& Connector0Correction = FVec3(0),
			const int32 LinearHardLambdaIndex = -1);

		void ApplyPositionConstraintSoft(
			const FReal Dt,
			const FReal JointStiffness,
			const FReal JointDamping,
			const bool bAccelerationMode,
			const FVec3& Axis,
			const FReal Delta,
			const FReal TargetVel,
			const FReal LambdaLimit,
			FReal& Lambda);

		void ApplyRotationConstraint(
			const FReal JointStiffness,
			const FVec3& Axis,
			const FReal Angle,
			const int32 AngularHardLambdaIndex = -1);

		void ApplyRotationConstraintKD(
			const int32 KIndex,
			const int32 DIndex,
			const FReal JointStiffness,
			const FVec3& Axis,
			const FReal Angle,
			const int32 AngularHardLambdaIndex = -1);

		void ApplyRotationConstraintDD(
			const FReal JointStiffness,
			const FVec3& Axis,
			const FReal Angle,
			const int32 AngularHardLambdaIndex = -1);

		void ApplyRotationConstraintSoft(
			const FReal Dt,
			const FReal JointStiffness,
			const FReal JointDamping,
			const bool bAccelerationMode,
			const FVec3& Axis,
			const FReal Angle,
			const FReal AngVelTarget,
			const FReal LambdaLimit,
			FReal& Lambda);

		void ApplyRotationConstraintSoftKD(
			const int32 KIndex,
			const int32 DIndex,
			const FReal Dt,
			const FReal JointStiffness,
			const FReal JointDamping,
			const bool bAccelerationMode,
			const FVec3& Axis,
			const FReal Angle,
			const FReal AngVelTarget,
			const FReal LambdaLimit,
			FReal& Lambda);

		void ApplyRotationConstraintSoftDD(
			const FReal Dt,
			const FReal JointStiffness,
			const FReal JointDamping,
			const bool bAccelerationMode,
			const FVec3& Axis,
			const FReal Angle,
			const FReal AngVelTarget,
			const FReal LambdaLimit,
			FReal& Lambda);

		void ApplyLockedRotationConstraints(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings,
			const bool bApplyTwist,
			const bool bApplySwing);

		void ApplyTwistConstraint(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings,
			const bool bUseSoftLimit);

		void ApplyConeConstraint(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings,
			const bool bUseSoftLimit);

		// One Swing axis is free, and the other locked. This applies the lock: Body1 Twist axis is confined to a plane.
		void ApplySingleLockedSwingConstraint(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings,
			const EJointAngularConstraintIndex SwingConstraintIndex,
			const bool bUseSoftLimit);

		// One Swing axis is free, and the other limited. This applies the limit: Body1 Twist axis is confined to space between two cones.
		void ApplyDualConeSwingConstraint(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings,
			const EJointAngularConstraintIndex SwingConstraintIndex,
			const bool bUseSoftLimit);

		// One swing axis is locked, the other limited or locked. This applies the Limited axis (ApplyDualConeSwingConstraint is used for the locked axis).
		void ApplySwingConstraint(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings,
			const EJointAngularConstraintIndex SwingConstraintIndex,
			const bool bUseSoftLimit);

		void ApplySwingTwistDrives(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings,
			const bool bTwistDriveEnabled,
			const bool bSwing1DriveEnabled,
			const bool bSwing2DriveEnabled);

		void ApplySLerpDrive(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		void ApplyPointPositionConstraintKD(
			const int32 KIndex,
			const int32 DIndex,
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		void ApplyPointPositionConstraintDD(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		void ApplySphericalPositionConstraint(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		void ApplyCylindricalPositionConstraint(
			const FReal Dt,
			const int32 AxisIndex,
			const EJointMotionType AxialMotion,
			const EJointMotionType RadialMotion,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		void ApplyPlanarPositionConstraint(
			const FReal Dt,
			const int32 AxisIndex,
			const EJointMotionType AxialMotion,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		void ApplyPositionDrive(
			const FReal Dt,
			const int32 AxisIndex,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings,
			const FVec3& Axis,
			const FReal DeltaPos,
			const FReal DeltaVel);

		void ApplyPositionProjection(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings,
			FVec3& DP1,
			FVec3& DR1);

		void ApplyRotationProjection(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings,
			FVec3& DP1,
			FVec3& DR1);

		void ApplyPointProjection(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings,
			const FReal Alpha,
			FVec3& NetDP1,
			FVec3& NetDR1);

		void ApplySphereProjection(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings,
			const FReal Alpha,
			FVec3& NetDP1,
			FVec3& NetDR1);

		void ApplyTranslateProjection(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings,
			const FReal Alpha,
			FVec3& NetDP1,
			FVec3& NetDR1);

		void ApplyConeProjection(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings,
			const FReal Alpha,
			const bool bPositionLocked,
			FVec3& NetDP1,
			FVec3& NetDR1);

		void ApplySwingProjection(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings,
			const EJointAngularConstraintIndex SwingConstraintIndex,
			const FReal Alpha,
			const bool bPositionLocked,
			FVec3& NetDP1,
			FVec3& NetDR1);

		void ApplySingleLockedSwingProjection(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings,
			const EJointAngularConstraintIndex SwingConstraintIndex,
			const FReal Alpha,
			const bool bPositionLocked,
			FVec3& NetDP1,
			FVec3& NetDR1);

		void ApplyDoubleLockedSwingProjection(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings,
			const FReal Alpha,
			const bool bPositionLocked,
			FVec3& NetDP1,
			FVec3& NetDR1);

		void ApplyDualConeSwingProjection(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings,
			const EJointAngularConstraintIndex SwingConstraintIndex,
			const FReal Alpha,
			const bool bPositionLocked,
			FVec3& NetDP1,
			FVec3& NetDR1);

		void ApplyTwistProjection(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings,
			const FReal Alpha,
			const bool bPositionLocked,
			FVec3& NetDP1,
			FVec3& NetDR1);

		void ApplyVelocityProjection(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings,
			const FReal Alpha,
			const FVec3& DP1,
			const FVec3& DR1);

		void ApplyLinearVelocityConstraint(
			const FReal Stiffness,
			const FVec3& Axis,
			const FVec3& Connector0Correction = FVec3(0),
			const FReal TargetVel = 0);

		void ApplyPointVelocityConstraint(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		void ApplySphericalVelocityConstraint(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		void ApplyCylindricalVelocityConstraint(
			const FReal Dt,
			const int32 AxisIndex,
			const EJointMotionType AxialMotion,
			const EJointMotionType RadialMotion,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		void ApplyPlanarVelocityConstraint(
			const FReal Dt,
			const int32 AxisIndex,
			const EJointMotionType AxialMotion,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		void ApplyTwistVelocityConstraint(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings,
			const bool bUseSoftLimit);

		void ApplyConeVelocityConstraint(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings,
			const bool bUseSoftLimit);

		// One Swing axis is free, and the other locked. This applies the lock: Body1 Twist axis is confined to a plane.
		void ApplySingleLockedSwingVelocityConstraint(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings,
			const EJointAngularConstraintIndex SwingConstraintIndex,
			const bool bUseSoftLimit);

		// One Swing axis is free, and the other limited. This applies the limit: Body1 Twist axis is confined to space between two cones.
		void ApplyDualConeSwingVelocityConstraint(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings,
			const EJointAngularConstraintIndex SwingConstraintIndex,
			const bool bUseSoftLimit);

		// One swing axis is locked, the other limited or locked. This applies the Limited axis (ApplyDualConeSwingConstraint is used for the locked axis).
		void ApplySwingVelocityConstraint(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings,
			const EJointAngularConstraintIndex SwingConstraintIndex,
			const bool bUseSoftLimit);

		void ApplyAngularVelocityConstraint(
			const FReal Stiffness,
			const FVec3& Axis,
			const FReal TargetVel = 0.0f);

		void ApplyLockedRotationVelocityConstraints(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings,
			const bool bApplyTwist,
			const bool bApplySwing);

		void CalculateLinearConstraintPadding(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings,
			const FReal Restitution,
			const int32 AxisIndex,
			const FVec3 Axis,
			FReal& InOutPos);

		void CalculateAngularConstraintPadding(
			const FReal Dt,
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings,
			const FReal Restitution,
			const EJointAngularConstraintIndex ConstraintIndex,
			const FVec3 Axis,
			FReal& InOutAngle);

		// Calculate linear velocities along constraint axes based on the constraint type in JointSettings. CV is the relative linear velocity of joint connectors.
		void CalculateConstraintAxisLinearVelocities(
			const FPBDJointSettings& JointSettings,
			FVec3& ConstraintAxisLinearVelocities) const;

		void CalculateSphericalConstraintAxisLinearVelocities(
			const FPBDJointSettings& JointSettings,
			FVec3& ConstraintAxisLinearVelocities
			) const;

		void CalculateCylindricalConstraintAxisLinearVelocities(
			const int32 AxisIndex,
			const EJointMotionType AxialMotion,
			const EJointMotionType RadialMotion,
			const FPBDJointSettings& JointSettings,
			FVec3& ConstraintAxisLinearVelocities) const;

		void CalculatePlanarConstraintAxisLinearVelocities(
			const int32 AxisIndex,
			const EJointMotionType AxialMotion,
			const FPBDJointSettings& JointSettings,
			FVec3& ConstraintAxisLinearVelocities) const;

		// Calculate angular velocities along constraint axes based on the constraint type in JointSettings. CW is the relative angular velocity of joint connectors.
		void CalculateConstraintAxisAngularVelocities(
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings,
			FVec3& ConstraintAxisAngularVelocities
			) const;

		void CalculateTwistConstraintAxisAngularVelocities(
			const FPBDJointSettings& JointSettings,
			FVec3& ConstraintAxisAngularVelocities) const;

		void CalculateConeConstraintAxisAngularVelocities(
			const FPBDJointSettings& JointSettings,
			FVec3& ConstraintAxisAngularVelocities) const;

		void CalculateDualConeSwingConstraintAxisAngularVelocities(
			const FPBDJointSettings& JointSettings,
			const EJointAngularConstraintIndex SwingConstraintIndex,
			FVec3& ConstraintAxisAngularVelocities) const;

		void CalculateSwingConstraintAxisAngularVelocities(
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings,
			const EJointAngularConstraintIndex SwingConstraintIndex,
			FVec3& ConstraintAxisAngularVelocities) const;

		// The cached body state on which the joint operates
		FConstraintSolverBody SolverBodies[MaxConstrainedBodies];

		// Local-space constraint settings
		FRigidTransform3 LocalConnectorXs[MaxConstrainedBodies];	// Local(CoM)-space joint connector transforms

		// World-space constraint settings
		FVec3 ConnectorXs[MaxConstrainedBodies];			// World-space joint connector positions
		FRotation3 ConnectorRs[MaxConstrainedBodies];		// World-space joint connector rotations

		// XPBD Initial iteration world-space body state
		FVec3 InitConnectorXs[MaxConstrainedBodies];		// World-space joint connector positions
		FRotation3 InitConnectorRs[MaxConstrainedBodies];	// World-space joint connector rotations

		// Conditioned InvM and InvI
		FReal InvMScales[MaxConstrainedBodies];
		FReal ConditionedInvMs[MaxConstrainedBodies];
		FVec3 ConditionedInvILs[MaxConstrainedBodies];		// Local-space
		FReal InvMs[MaxConstrainedBodies];
		FMatrix33 InvIs[MaxConstrainedBodies];				// World-space

		// Accumulated Impulse and AngularImpulse (Impulse * Dt since they are mass multiplied position corrections)
		FVec3 NetLinearImpulse;
		FVec3 NetAngularImpulse;

		FReal LinearViolationSq;
		FReal AngularViolation;

		// Lagrange multipliers of the position constraints.
		// Currently these are only used in ApplyCylindricalVelocityConstraints
		FVec3 LinearHardLambda;
		// Lagrange multipliers of the rotation constraints
		// Currently these are used in ApplyAngularVelocityConstraints
		FVec3 AngularHardLambda;

		// XPBD Accumulators (net impulse for each soft constraint/drive)
		FReal LinearSoftLambda;
		FReal TwistSoftLambda;
		FReal SwingSoftLambda;
		FVec3 LinearDriveLambdas;
		FVec3 RotationDriveLambdas;

		// Solver stiffness - increased over iterations for stability
		// \todo(chaos): remove Stiffness from SolverSettings (because it is not a solver constant)
		FReal SolverStiffness;

		// Tolerances below which we stop solving
		FReal PositionTolerance;					// Distance error below which we consider a constraint or drive solved
		FReal AngleTolerance;						// Angle error below which we consider a constraint or drive solved

		// Tracking whether the solver is resolved
		FVec3 LastPs[MaxConstrainedBodies];			// Positions at the beginning of the iteration
		FRotation3 LastQs[MaxConstrainedBodies];	// Rotations at the beginning of the iteration
		FVec3 InitConstraintAxisLinearVelocities; // Linear velocities along the constraint axes at the begining of the frame, used by restitution
		FVec3 InitConstraintAxisAngularVelocities; // Angular velocities along the constraint axes at the begining of the frame, used by restitution

		bool bIsBroken;
		bool bIsViolating;
	};

}