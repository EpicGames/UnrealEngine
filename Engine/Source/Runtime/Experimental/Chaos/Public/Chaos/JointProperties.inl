// Copyright Epic Games, Inc. All Rights Reserved.

//CHAOS_INNER_JOINT_PROPERTY(OuterProp, FuncName, InnerName, InnerType)							
CHAOS_INNER_JOINT_PROPERTY(JointSettings, JointTransforms, ConnectorTransforms, FTransformPair)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, CollisionEnabled, bCollisionEnabled, bool)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, ProjectionEnabled, bProjectionEnabled, bool)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, ShockPropagationEnabled, bShockPropagationEnabled, bool)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, MassConditioningEnabled, bMassConditioningEnabled, bool)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, UseLinearSolver, bUseLinearSolver, bool)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, ProjectionLinearAlpha, LinearProjection, FReal)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, ProjectionAngularAlpha, AngularProjection, FReal)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, ProjectionLinearTolerance, TeleportDistance, FReal)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, ProjectionAngularTolerance, TeleportAngle, FReal)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, ShockPropagationAlpha, ShockPropagation, FReal)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, ParentInvMassScale, ParentInvMassScale, FReal)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, LinearBreakForce, LinearBreakForce, FReal)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, LinearViolationCallbackThreshold, LinearViolationCallbackThreshold, FReal)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, LinearPlasticityLimit, LinearPlasticityLimit, FReal)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, LinearPlasticityType, LinearPlasticityType, EPlasticityType)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, AngularBreakTorque, AngularBreakTorque, FReal)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, AngularViolationCallbackThreshold, AngularViolationCallbackThreshold, FReal)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, AngularPlasticityLimit, AngularPlasticityLimit, FReal)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, UserData, UserData, void*)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, LinearPositionDriveXEnabled, bLinearPositionDriveEnabled[0], bool)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, LinearPositionDriveYEnabled, bLinearPositionDriveEnabled[1], bool)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, LinearPositionDriveZEnabled, bLinearPositionDriveEnabled[2], bool)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, LinearDrivePositionTarget, LinearDrivePositionTarget, FVec3)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, LinearVelocityDriveXEnabled, bLinearVelocityDriveEnabled[0], bool)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, LinearVelocityDriveYEnabled, bLinearVelocityDriveEnabled[1], bool)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, LinearVelocityDriveZEnabled, bLinearVelocityDriveEnabled[2], bool)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, LinearDriveVelocityTarget, LinearDriveVelocityTarget, FVec3)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, LinearDriveForceMode, LinearDriveForceMode, EJointForceMode)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, LinearMotionTypesX, LinearMotionTypes[0], EJointMotionType)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, LinearMotionTypesY, LinearMotionTypes[1], EJointMotionType)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, LinearMotionTypesZ, LinearMotionTypes[2], EJointMotionType)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, LinearDriveStiffness, LinearDriveStiffness, FVec3)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, LinearDriveDamping, LinearDriveDamping, FVec3)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, LinearDriveMaxForce, LinearDriveMaxForce, FVec3)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, ContactTransferScale, ContactTransferScale, FReal)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, AngularSLerpPositionDriveEnabled, bAngularSLerpPositionDriveEnabled, bool)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, AngularTwistPositionDriveEnabled, bAngularTwistPositionDriveEnabled, bool)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, AngularSwingPositionDriveEnabled, bAngularSwingPositionDriveEnabled, bool)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, AngularDrivePositionTarget, AngularDrivePositionTarget, FRotation3)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, AngularSLerpVelocityDriveEnabled, bAngularSLerpVelocityDriveEnabled, bool)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, AngularTwistVelocityDriveEnabled, bAngularTwistVelocityDriveEnabled, bool)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, AngularSwingVelocityDriveEnabled, bAngularSwingVelocityDriveEnabled, bool)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, AngularDriveVelocityTarget, AngularDriveVelocityTarget, FVec3)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, AngularDriveForceMode, AngularDriveForceMode, EJointForceMode)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, AngularMotionTypesX, AngularMotionTypes[0], EJointMotionType)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, AngularMotionTypesY, AngularMotionTypes[1], EJointMotionType)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, AngularMotionTypesZ, AngularMotionTypes[2], EJointMotionType)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, AngularDriveStiffness, AngularDriveStiffness, FVec3)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, AngularDriveDamping, AngularDriveDamping, FVec3)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, AngularDriveMaxTorque, AngularDriveMaxTorque, FVec3)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, Stiffness, Stiffness, FReal)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, SoftLinearLimitsEnabled, bSoftLinearLimitsEnabled, bool)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, SoftTwistLimitsEnabled, bSoftTwistLimitsEnabled, bool)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, SoftSwingLimitsEnabled, bSoftSwingLimitsEnabled, bool)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, LinearSoftForceMode, LinearSoftForceMode, EJointForceMode)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, AngularSoftForceMode, AngularSoftForceMode, EJointForceMode)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, SoftLinearStiffness, SoftLinearStiffness, FReal)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, SoftLinearDamping, SoftLinearDamping, FReal)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, SoftTwistStiffness, SoftTwistStiffness, FReal)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, SoftTwistDamping, SoftTwistDamping, FReal)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, SoftSwingStiffness, SoftSwingStiffness, FReal)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, SoftSwingDamping, SoftSwingDamping, FReal)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, LinearLimit, LinearLimit, FReal)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, AngularLimits, AngularLimits, FVec3)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, LinearContactDistance, LinearContactDistance, FReal)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, TwistContactDistance, TwistContactDistance, FReal)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, SwingContactDistance, SwingContactDistance, FReal)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, LinearRestitution, LinearRestitution, FReal)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, TwistRestitution, TwistRestitution, FReal)
CHAOS_INNER_JOINT_PROPERTY(JointSettings, SwingRestitution, SwingRestitution, FReal)

#undef CHAOS_INNER_JOINT_PROPERTY