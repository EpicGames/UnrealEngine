// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/ArrayCollectionArray.h"
#include "Chaos/GeometryParticles.h"
#include "Chaos/KinematicTargets.h"

namespace Chaos
{

template<class T, int d, EGeometryParticlesSimType SimType>
class TKinematicGeometryParticlesImp : public TGeometryParticlesImp<T, d, SimType>
{
  public:
	TKinematicGeometryParticlesImp()
	    : TGeometryParticlesImp<T, d, SimType>()
	{
		this->MParticleType = EParticleType::Kinematic;
		TArrayCollection::AddArray(&MV);
		TArrayCollection::AddArray(&MW);
		TArrayCollection::AddArray(&KinematicTargets);
	}
	TKinematicGeometryParticlesImp(const TKinematicGeometryParticlesImp<T, d, SimType>& Other) = delete;
	TKinematicGeometryParticlesImp(TKinematicGeometryParticlesImp<T, d, SimType>&& Other)
	    : TGeometryParticlesImp<T, d, SimType>(MoveTemp(Other)), MV(MoveTemp(Other.MV)), MW(MoveTemp(Other.MW)), KinematicTargets(MoveTemp(Other.KinematicTargets))
	{
		this->MParticleType = EParticleType::Kinematic;
		TArrayCollection::AddArray(&MV);
		TArrayCollection::AddArray(&MW);
		TArrayCollection::AddArray(&KinematicTargets);
	}
	virtual ~TKinematicGeometryParticlesImp() = default;

	UE_DEPRECATED(5.4, "Use GetV instead")
	const TVector<T, d> V(const int32 Index) const { return TVector<T, d>(MV[Index]); }
	UE_DEPRECATED(5.4, "Use GetV or SetV instead")
	TVector<T, d> V(const int32 Index) { return TVector<T, d>(MV[Index]); }
	const TVector<T, d> GetV(const int32 Index) const { return TVector<T, d>(MV[Index]); }
	void SetV(const int32 Index, const TVector<T, d>& InV) { MV[Index] = InV; }
	const TVector<FRealSingle, d> GetVf(const int32 Index) const { return MV[Index]; }
	void SetVf(const int32 Index, const TVector<FRealSingle, d>& InV) { MV[Index] = InV; }

	UE_DEPRECATED(5.4, "Use GetW instead")
	const TVector<T, d> W(const int32 Index) const { return TVector<T, d>(MW[Index]); }
	UE_DEPRECATED(5.4, "Use GetW or SetW instead")
	TVector<T, d> W(const int32 Index) { return TVector<T, d>(MW[Index]); }
	const TVector<T, d> GetW(const int32 Index) const { return TVector<T, d>(MW[Index]); }
	void SetW(const int32 Index, const TVector<T, d>& InW) { MW[Index] = InW; }
	const TVector<FRealSingle, d> GetWf(const int32 Index) const { return MW[Index]; }
	void SetWf(const int32 Index, const TVector<FRealSingle, d>& InW) { MW[Index] = InW; }

	const FKinematicTarget& KinematicTarget(const int32 Index) const { return KinematicTargets[Index]; }
	FKinematicTarget& KinematicTarget(const int32 Index) { return KinematicTargets[Index]; }

	FString ToString(int32 index) const
	{
		FString BaseString = TGeometryParticlesImp<T, d, SimType>::ToString(index);
		return FString::Printf(TEXT("%s, MV:%s, MW:%s"), *BaseString, *GetV(index).ToString(), *GetW(index).ToString());
	}

	typedef TKinematicGeometryParticleHandle<T, d> THandleType;
	const THandleType* Handle(int32 Index) const;

	//cannot be reference because double pointer would allow for badness, but still useful to have non const access to handle
	THandleType* Handle(int32 Index);

	virtual void Serialize(FChaosArchive& Ar) override
	{
		TGeometryParticlesImp<T, d, SimType>::Serialize(Ar);
		
		Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);
		if (Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) >= FUE5ReleaseStreamObjectVersion::SinglePrecisonParticleDataPT)
		{
			Ar << MV << MW;
		}
		else
		{
			TArrayCollectionArray<TVector<FReal, d>> VDouble;
			VDouble.Resize(MV.Num());
			for (int32 Index = 0; Index < MV.Num(); ++Index)
			{
				VDouble[Index] = MV[Index];
			}
			TArrayCollectionArray<TVector<FReal, d>> WDouble;
			WDouble.Resize(MW.Num());
			for (int32 Index = 0; Index < MW.Num(); ++Index)
			{
				WDouble[Index] = MW[Index];
			}

			Ar << VDouble << WDouble;

			MV.Resize(VDouble.Num());
			for (int32 Index = 0; Index < VDouble.Num(); ++Index)
			{
				MV[Index] = VDouble[Index];
			}
			MW.Resize(WDouble.Num());
			for (int32 Index = 0; Index < WDouble.Num(); ++Index)
			{
				MW[Index] = WDouble[Index];
			}
		}

		Ar.UsingCustomVersion(FExternalPhysicsCustomObjectVersion::GUID);
		if(Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) >= FExternalPhysicsCustomObjectVersion::KinematicTargets)
		{
			Ar << KinematicTargets;
		}
	}

	FORCEINLINE TArray<TVector<FRealSingle, d>>& AllV() { return MV; }
	FORCEINLINE TArray<TVector<FRealSingle, d>>& AllW() { return MW; }
	FORCEINLINE TArray<FKinematicTarget>& AllKinematicTargets() { return KinematicTargets; }

  private:
	TArrayCollectionArray<TVector<FRealSingle, d>> MV;
	TArrayCollectionArray<TVector<FRealSingle, d>> MW;
	TArrayCollectionArray<FKinematicTarget> KinematicTargets;
};

template <typename T, int d, EGeometryParticlesSimType SimType>
FChaosArchive& operator<<(FChaosArchive& Ar, TKinematicGeometryParticlesImp<T, d, SimType>& Particles)
{
	Particles.Serialize(Ar);
	return Ar;
}



template <typename T, int d>
using TKinematicGeometryParticles = TKinematicGeometryParticlesImp<T, d, EGeometryParticlesSimType::RigidBodySim>;

using FKinematicGeometryParticles = TKinematicGeometryParticles<FReal, 3>;

template <typename T, int d>
using TKinematicGeometryClothParticles = TKinematicGeometryParticlesImp<T, d, EGeometryParticlesSimType::Other>;

using FKinematicGeometryClothParticles UE_DEPRECATED(5.0, "FKinematicGeometryClothParticles is deprecated, use Softs::FSolverRigidParticles instead.") = TKinematicGeometryClothParticles<FReal, 3>;
}

