// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// HEADER_UNIT_SKIP - Internal

#include "Chaos/ChaosArchive.h"
#include "Chaos/Transform.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/FortniteReleaseBranchCustomObjectVersion.h"
#include "UObject/PhysicsObjectVersion.h"

namespace Chaos
{
	/**
	 * Controls how a kinematic body is integrated each Evolution Advance
	 */
	enum class EKinematicTargetMode
	{
		None,			/** Particle does not move and no data is changed */
		Reset,			/** Particle does not move, velocity and angular velocity are zeroed, then mode is set to "None". */
		Position,		/** Particle is moved to Kinematic Target transform, velocity and angular velocity updated to reflect the change, then mode is set to "Reset". */
		Velocity,		/** Particle is moved based on velocity and angular velocity, mode remains as "Velocity" until changed. */
	};

	class FKinematicTarget;

	template<class T, int d>
	using TKinematicTarget UE_DEPRECATED(5.5, "Deprecated. this class is to be deleted, use class FKinematicTarget instead") = FKinematicTarget;

	/**
	 * Data used to integrate kinematic bodies
	 */
	class FKinematicTarget
	{
	public:

		static FKinematicTarget MakePositionTarget(const FRigidTransform3& InTransform)
		{
			return FKinematicTarget(InTransform);
		}

		static FKinematicTarget MakePositionTarget(const FVec3& InPosition, const FRotation3f& InRotation)
		{
			return FKinematicTarget(InPosition, InRotation);
		}

		FKinematicTarget()
			: Rotation(FRotation3f::FromIdentity())
			, Position(0)
			, Mode(EKinematicTargetMode::None)
		{
		}

		/** Whether this kinematic target has been set (either velocity or position mode) */
		bool IsSet() const { return (Mode == EKinematicTargetMode::Position) || (Mode == EKinematicTargetMode::Velocity); }

		/** Get the kinematic target mode */
		EKinematicTargetMode GetMode() const { return Mode; }

		UE_DEPRECATED(5.5, "This method is Deprecated and it will be removed in a future release. Use GetTransform instead")
		/** Get the target transform (asserts if not in Position mode) */
		FRigidTransform3 GetTarget() const { check(Mode == EKinematicTargetMode::Position); return {Position, Rotation}; }

		UE_DEPRECATED(5.5, "This method is Deprecated and it will be removed in a future release. Use GetPosition instead")
		/** Get the target position (asserts if not in Position mode) */
		FVec3 GetTargetPosition() const { check(Mode == EKinematicTargetMode::Position); return Position; }

		UE_DEPRECATED(5.5, "This method is Deprecated and it will be removed in a future release. Use GetRotation instead")
		/** Get the target rotation (asserts if not in Position mode) */
		FRotation3 GetTargetRotation() const { check(Mode == EKinematicTargetMode::Position); return Rotation; }

		/** Get the target transform (asserts if not in Position mode) */
		FRigidTransform3 GetTransform() const { check(Mode == EKinematicTargetMode::Position); return {Position, Rotation}; }

		/** Get the target position (asserts if not in Position mode) */
		FVec3 GetPosition() const { check(Mode == EKinematicTargetMode::Position); return Position; }

		/** Get the target rotation (asserts if not in Position mode) */
		FRotation3f GetRotation() const { check(Mode == EKinematicTargetMode::Position); return Rotation; }

		/** Clear the kinematic target */
		void Clear()
		{
			Position = FVec3();
			Rotation = FRotation3f();
			Mode = EKinematicTargetMode::None;
		}

		/** Use transform target mode and set the transform target */
		void SetTargetMode(const FVec3& X, const FRotation3f& R)
		{
			Position = X;
			Rotation = R;
			Mode = EKinematicTargetMode::Position;
		}

		/** Use transform target mode and set the transform target */
		void SetTargetMode(const FRigidTransform3& InTarget)
		{
			Position = InTarget.GetLocation();
			Rotation = InTarget.GetRotation();
			Mode = EKinematicTargetMode::Position;
		}

		/** Use velocity target mode */
		void SetVelocityMode() { Mode = EKinematicTargetMode::Velocity; }

		// For internal use only
		void SetMode(EKinematicTargetMode InMode) { Mode = InMode; }

		friend FChaosArchive& operator<<(FChaosArchive& Ar, FKinematicTarget& KinematicTarget)
		{
			Ar.UsingCustomVersion(FFortniteReleaseBranchCustomObjectVersion::GUID);
			Ar.UsingCustomVersion(FPhysicsObjectVersion::GUID);
			Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

			const bool bRemovedScaleFN = (Ar.CustomVer(FFortniteReleaseBranchCustomObjectVersion::GUID) >= FFortniteReleaseBranchCustomObjectVersion::ChaosKinematicTargetRemoveScale);
			const bool bRemovedScaleUE4 = (Ar.CustomVer(FPhysicsObjectVersion::GUID) >= FPhysicsObjectVersion::ChaosKinematicTargetRemoveScale);
			const bool bRotationStoredAsSinglePrecision = (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::ChaosStoreKinematicTargetRotationAsSinglePrecision);

			if (bRemovedScaleFN || bRemovedScaleUE4)
			{
				Ar << KinematicTarget.Position;

				if (bRotationStoredAsSinglePrecision)
				{
					Ar << KinematicTarget.Rotation;
				}
				else
				{
					FRotation3 RotationDoublePrecision;
					if (Ar.IsLoading())
					{
						Ar << RotationDoublePrecision;
						KinematicTarget.Rotation = RotationDoublePrecision;
					}
					else
					{
						RotationDoublePrecision = KinematicTarget.Rotation;
						Ar << RotationDoublePrecision;
					}
				}
				
				Ar << KinematicTarget.Mode;
			}
			else
			{
				FRigidTransform3 Transform;
				Ar << Transform << KinematicTarget.Mode;

				KinematicTarget.Position = FVec3(Transform.GetLocation());
				KinematicTarget.Rotation = TRotation3<FRealSingle>(Transform.GetRotation());
			}

			return Ar;
		}

		bool IsEqual(const FKinematicTarget& other) const
		{
			return (
				Mode == other.Mode &&
				Position == other.Position &&
				Rotation == other.Rotation
				);
		}

		template <typename TOther>
		bool IsEqual(const TOther& other) const
		{
			return IsEqual(other.KinematicTarget());
		}

		bool IsNearlyEqual(const FKinematicTarget& other, FReal Leniency = UE_KINDA_SMALL_NUMBER) const
		{
			return Mode == other.Mode &&
				FVec3::IsNearlyEqual(Position, other.Position, Leniency) &&
				FRotation3f::IsNearlyEqual(Rotation, other.Rotation, static_cast<FRealSingle>(Leniency));
		}

		template <typename TOther>
		bool IsNearlyEqual(const TOther& other, FReal Leniency = UE_KINDA_SMALL_NUMBER) const
		{
			return IsNearlyEqual(other.KinematicTarget(), Leniency);
		}

		bool operator==(const FKinematicTarget& other) const
		{
			return IsEqual(other);
		}

		template <typename TOther>
		void CopyFrom(const TOther& Other)
		{
			Position = Other.KinematicTarget().Position;
			Rotation = Other.KinematicTarget().Rotation;
			Mode = Other.KinematicTarget().Mode;
		}

		static FKinematicTarget ZeroValue()
		{
			return FKinematicTarget();
		}

	private:
		explicit FKinematicTarget(const FRigidTransform3& InTransform)
			: Rotation(InTransform.GetRotation())
			, Position(InTransform.GetTranslation())
			, Mode(EKinematicTargetMode::Position)
		{
		}

		FKinematicTarget(const FVec3& InPosition, const FRotation3f& InRotation)
			: Rotation(InRotation)
			, Position(InPosition)
			, Mode(EKinematicTargetMode::Position)
		{
		}

		FRotation3f Rotation;
		FVec3 Position;
		EKinematicTargetMode Mode;
	};

}

