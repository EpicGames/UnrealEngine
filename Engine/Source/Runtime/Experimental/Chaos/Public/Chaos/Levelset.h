// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/ArrayND.h"
#include "Chaos/Box.h"
#include "Chaos/ImplicitObject.h"
#include "Chaos/ImplicitObjectScaled.h"
#include "Chaos/Particles.h"
#include "Chaos/UniformGrid.h"

namespace Chaos { class FErrorReporter; }

struct FKLevelSetElem;

namespace Chaos
{

struct FMTDInfo;

class FTriangleMesh;

template<class T, int D>
class TPlane;

class FCapsule;

class FConvex;

class FLevelSet final : public FImplicitObject
{
  public:
	using FImplicitObject::SignedDistance;

	CHAOS_API FLevelSet(FErrorReporter& ErrorReporter, const TUniformGrid<FReal, 3>& InGrid, const FParticles& InParticles, const FTriangleMesh& Mesh, const int32 BandWidth = 0);
	CHAOS_API FLevelSet(FErrorReporter& ErrorReporter, const TUniformGrid<FReal, 3>& InGrid, const FImplicitObject& InObject, const int32 BandWidth = 0, const bool bUseObjectPhi = false);
#if COMPILE_WITHOUT_UNREAL_SUPPORT
	FLevelSet(std::istream& Stream);
#endif
	CHAOS_API FLevelSet(TUniformGrid<FReal, 3>&& Grid, TArrayND<FReal, 3>&& Phi, int32 BandWidth);
	FLevelSet(const FLevelSet& Other) = delete;
	CHAOS_API FLevelSet(FLevelSet&& Other);
	CHAOS_API virtual ~FLevelSet();

	CHAOS_API virtual Chaos::FImplicitObjectPtr CopyGeometry() const override;
	CHAOS_API virtual Chaos::FImplicitObjectPtr CopyGeometryWithScale(const FVec3& Scale) const override;

#if COMPILE_WITHOUT_UNREAL_SUPPORT
	CHAOS_API void Write(std::ostream& Stream) const;
#endif
	CHAOS_API virtual FReal PhiWithNormal(const FVec3& x, FVec3& Normal) const override;
	CHAOS_API FReal SignedDistance(const FVec3& x) const;

	virtual const FAABB3 BoundingBox() const override { return MLocalBoundingBox; }

	// Returns a const ref to the underlying phi grid
	const TArrayND<FReal, 3>& GetPhiArray() const { return MPhi; }

	// Returns a const ref to the underlying grid of normals
	const TArrayND<FVec3, 3>& GetNormalsArray() const { return MNormals; }

	// Returns a const ref to the underlying grid structure
	const TUniformGrid<FReal, 3>& GetGrid() const { return MGrid; }

	FORCEINLINE void Shrink(const FReal Value)
	{
		for (int32 i = 0; i < MGrid.Counts().Product(); ++i)
		{
			MPhi[i] += Value;
		}
	}

	FORCEINLINE static constexpr EImplicitObjectType StaticType()
	{
		return ImplicitObjectType::LevelSet;
	}

	FORCEINLINE void SerializeImp(FArchive& Ar)
	{
		FImplicitObject::SerializeImp(Ar);
		Ar << MGrid;
		Ar << MPhi;
		Ar << MNormals;
		TBox<FReal, 3>::SerializeAsAABB(Ar, MLocalBoundingBox);
		TBox<FReal, 3>::SerializeAsAABB(Ar, MOriginalLocalBoundingBox);
		Ar << MBandWidth;
	}

	virtual void Serialize(FChaosArchive& Ar) override
	{
		SerializeImp(Ar);
	}

	virtual void Serialize(FArchive& Ar) override
	{
		SerializeImp(Ar);
	}

	/** 
	 * Estimate the volume bounded by the zero'th isocontour of the level set.
	 * #BGTODO - We can generate a more accurate pre-calculated volume during generation as this method still under
	 * estimates the actual volume of the surface.
	 */
	FReal ApproximateNegativeMaterial() const
	{
		const FVec3& CellDim = MGrid.Dx();
		const FReal AvgRadius = (CellDim[0] + CellDim[1] + CellDim[2]) / (FReal)3;
		const FReal CellVolume = CellDim.Product();
		FReal Volume = 0.0;
		for (int32 Idx = 0; Idx < MPhi.Num(); ++Idx)
		{
			const FReal Phi = MPhi[Idx];
			if (Phi <= 0.0)
			{
				FReal CellRadius = AvgRadius - FMath::Abs(Phi);

				if(CellRadius > UE_KINDA_SMALL_NUMBER)
				{
					const FReal Scale = FMath::Min((FReal)1, CellRadius / AvgRadius);
					Volume += CellVolume * Scale;
				}
				else
				{
					Volume += CellVolume;
				}
			}
		}
		return Volume;
	}

	CHAOS_API bool ComputeMassProperties(FReal& OutVolume, FVec3& OutCOM, FMatrix33& OutInertia, FRotation3& OutRotationOfMass) const;

	CHAOS_API FReal ComputeLevelSetError(const FParticles& InParticles, const TArray<FVec3>& Normals, const FTriangleMesh& Mesh, FReal& AngleError, FReal& MaxDistError);

	// Output a mesh and level set as obj files
	CHAOS_API void OutputDebugData(FErrorReporter& ErrorReporter, const FParticles& InParticles, const TArray<FVec3>& Normals, const FTriangleMesh& Mesh, const FString FileName);

	CHAOS_API bool CheckData(FErrorReporter& ErrorReporter, const FParticles& InParticles, const FTriangleMesh& Mesh, const TArray<FVec3> &Normals);

	// Used to generate a simple debug surface
	CHAOS_API void GetZeroIsosurfaceGridCellFaces(TArray<FVector3f>& Vertices, TArray<FIntVector>& Tris) const;
	CHAOS_API void GetInteriorCells(TArray<TVec3<int32>>& InteriorCells, const FReal InteriorThreshold) const;

	virtual uint32 GetTypeHash() const override
	{
		uint32 Result = 0;

		const int32 NumValues = MPhi.Num();

		for(int32 Index = 0; Index < NumValues; ++Index)
		{
			Result = HashCombine(Result, ::GetTypeHash(MPhi[Index]));
		}

		return Result;
	}


	CHAOS_API bool SweepGeom(const FSphere& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, const FReal Thickness = 0, const bool bComputeMTD = false) const;
	CHAOS_API bool SweepGeom(const TBox<FReal, 3>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, const FReal Thickness = 0, const bool bComputeMTD = false) const;
	CHAOS_API bool SweepGeom(const FCapsule& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, const FReal Thickness = 0, const bool bComputeMTD = false) const;
	CHAOS_API bool SweepGeom(const FConvex& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, const FReal Thickness = 0, const bool bComputeMTD = false) const;

	CHAOS_API bool SweepGeom(const TImplicitObjectScaled<FSphere>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, const FReal Thickness = 0, const bool bComputeMTD = false) const;
	CHAOS_API bool SweepGeom(const TImplicitObjectScaled<TBox<FReal, 3>>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, const FReal Thickness = 0, const bool bComputeMTD = false) const;
	CHAOS_API bool SweepGeom(const TImplicitObjectScaled<FCapsule>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, const FReal Thickness = 0, const bool bComputeMTD = false) const;
	CHAOS_API bool SweepGeom(const TImplicitObjectScaled<FConvex>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, const FReal Thickness = 0, const bool bComputeMTD = false) const;

	template<typename QueryGeomType>
	bool SweepGeomImp(const QueryGeomType& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, const FReal Thickness, const bool bComputeMTD) const;

	CHAOS_API bool OverlapGeom(const FSphere& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD = nullptr) const;
	CHAOS_API bool OverlapGeom(const TBox<FReal, 3>& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD = nullptr) const;
	CHAOS_API bool OverlapGeom(const FCapsule& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD = nullptr) const;
	CHAOS_API bool OverlapGeom(const FConvex& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD = nullptr) const;

	CHAOS_API bool OverlapGeom(const TImplicitObjectScaled<FSphere>& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD = nullptr) const;
	CHAOS_API bool OverlapGeom(const TImplicitObjectScaled<TBox<FReal, 3>>& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD = nullptr) const;
	CHAOS_API bool OverlapGeom(const TImplicitObjectScaled<FCapsule>& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD = nullptr) const;
	CHAOS_API bool OverlapGeom(const TImplicitObjectScaled<FConvex>& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD = nullptr) const;

	template<typename QueryGeomType>
	bool OverlapGeomImp(const QueryGeomType& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD = nullptr) const;

  private:
	CHAOS_API bool ComputeDistancesNearZeroIsocontour(FErrorReporter& ErrorReporter, const FParticles& InParticles, const TArray<FVec3> &Normals, const FTriangleMesh& Mesh, TArrayND<bool, 3>& BlockedFaceX, TArrayND<bool, 3>& BlockedFaceY, TArrayND<bool, 3>& BlockedFaceZ, TArray<TVec3<int32>>& InterfaceIndices);
	CHAOS_API void ComputeDistancesNearZeroIsocontour(const FImplicitObject& Object, const TArrayND<FReal, 3>& ObjectPhi, TArray<TVec3<int32>>& InterfaceIndices);
	CHAOS_API void CorrectSign(const TArrayND<bool, 3>& BlockedFaceX, const TArrayND<bool, 3>& BlockedFaceY, const TArrayND<bool, 3>& BlockedFaceZ, TArray<TVec3<int32>>& InterfaceIndices);
	CHAOS_API FReal ComputePhi(const TArrayND<bool, 3>& Done, const TVec3<int32>& CellIndex);
	CHAOS_API void FillWithFastMarchingMethod(const FReal StoppingDistance, const TArray<TVec3<int32>>& InterfaceIndices);
	CHAOS_API void FloodFill(const TArrayND<bool, 3>& BlockedFaceX, const TArrayND<bool, 3>& BlockedFaceY, const TArrayND<bool, 3>& BlockedFaceZ, TArrayND<int32, 3>& Color, int32& NextColor);
	CHAOS_API void FloodFillFromCell(const TVec3<int32> CellIndex, const int32 NextColor, const TArrayND<bool, 3>& BlockedFaceX, const TArrayND<bool, 3>& BlockedFaceY, const TArrayND<bool, 3>& BlockedFaceZ, TArrayND<int32, 3>& Color);
	CHAOS_API bool IsIntersectingWithTriangle(const FParticles& Particles, const TVec3<int32>& Elements, const TPlane<FReal, 3>& TrianglePlane, const TVec3<int32>& CellIndex, const TVec3<int32>& PrevCellIndex);
	CHAOS_API void ComputeNormals();
	CHAOS_API void ComputeConvexity(const TArray<TVec3<int32>>& InterfaceIndices);
	
	CHAOS_API void ComputeNormals(const FParticles& InParticles, const FTriangleMesh& Mesh, const TArray<TVec3<int32>>& InterfaceIndices);

	TUniformGrid<FReal, 3> MGrid;
	TArrayND<FReal, 3> MPhi;
	TArrayND<FVec3, 3> MNormals;
	FAABB3 MLocalBoundingBox;
	FAABB3 MOriginalLocalBoundingBox;
	int32 MBandWidth;
private:
	FLevelSet() : FImplicitObject(EImplicitObject::HasBoundingBox, ImplicitObjectType::LevelSet) {}	//needed for serialization
	friend FImplicitObject;	//needed for serialization
	friend ::FKLevelSetElem; //needed for serialization
};

template <typename T, int d>
using TLevelSet UE_DEPRECATED(4.27, "Deprecated. this class is to be deleted, use FLevelSet instead") = FLevelSet;
}
