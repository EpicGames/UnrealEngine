// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NNEModelData.h"  
#include "NNERuntimeCPU.h" 

class FMLLevelSet;

namespace Chaos
{ 

// Class for MLLevelSet Weight Depended Models
class FMLLevelSetNeuralInference
{
  public:
	  FMLLevelSetNeuralInference() {}
	  CHAOS_API FMLLevelSetNeuralInference(TSharedPtr<UE::NNE::IModelCPU> NNEModel, TArray<TArray<int32>>& InModelWeightsShapes);
	  CHAOS_API FMLLevelSetNeuralInference(FMLLevelSetNeuralInference&& Other);
	  CHAOS_API FMLLevelSetNeuralInference(const FMLLevelSetNeuralInference& Other);
	  FMLLevelSetNeuralInference& operator=(const FMLLevelSetNeuralInference& Other);
	  FMLLevelSetNeuralInference* Copy() const;
	  void SetModelWeightsShapes(TArray<TArray<uint32>>& InModelWeightsShapes) { ModelWeightsShapes = InModelWeightsShapes; }
	  const bool IsValid() const { return ModelInstance.IsValid(); } 
	  void RunInference(TArray<float, TAlignedHeapAllocator<64>>& InputData, TArray<float, TAlignedHeapAllocator<64>>& OutputData, uint32 SingleInputSize, uint32 SingleOutputSize, TArray<TArray<float, TAlignedHeapAllocator<64>>>& ModelWeightsIn) const; // Batched Version of the RunMLModel. 
	  bool Serialize(FArchive& Ar);
 
  private:
	// Different instances of FMLLevelSetNeuralInference can share the same NNEModel (even with deep copy). 
	// Since each model has its own ModelInstance, this would not cause a problem during inference.
	TSharedPtr<UE::NNE::IModelCPU> NNEModel; 
	TSharedPtr<UE::NNE::IModelInstanceCPU> ModelInstance; //can this be TUniquePtr?
	TArray<TArray<uint32>> ModelWeightsShapes; 
	CHAOS_API FMLLevelSetNeuralInference(TSharedPtr<UE::NNE::IModelCPU> NNEModel); //needed for serialization
	
	friend FMLLevelSet;
};
}
