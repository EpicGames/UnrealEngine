// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/ArrayND.h"
#include "Chaos/Box.h"
#include "Chaos/ImplicitObject.h"
#include "Chaos/ImplicitObjectScaled.h"
#include "Chaos/MLLevelSetNeuralInference.h"
#include "Chaos/PBDSoftsEvolutionFwd.h" 

struct FKMLLevelSetElem;

namespace Chaos
{

struct FMLLevelSetNNEModelData
{
	TArray<int32> ModelArchitectureActivationNodeSizes;
	const FString MLModelWeightsString;
	const FString NNEModelPath; 
	TObjectPtr<UNNEModelData> NNEModelData;
};

struct FMLLevelSetImportData
{
	TArray<FName> ActiveBoneNames;
	TArray<FMLLevelSetNNEModelData> NNEModelDataArr; //Either {SignedDistanceModel} or {SignedDistanceModel, IncorrectZoneModel} 
	TArray<FVector3f> ActiveBonesReferenceRotations;
	TArray<FVector3f> ActiveBonesReferenceTranslations;
	const float SignedDistanceScaling;
	TArray<TArray<int32>> ActiveBonesRotationComponents;
	const FVector3f TrainingGridMin;
	TArray<FVector3f> TrainingGridAxesXYZ;
	FIntVector DebugGridResolution;
};

class FMLLevelSet final : public FImplicitObject
{
  public:
	CHAOS_API FMLLevelSet(FMLLevelSetImportData&& MLLevelSetImportData);

	FMLLevelSet(const FMLLevelSet& Other) = delete; 
	CHAOS_API FMLLevelSet(FMLLevelSet&& Other);
	CHAOS_API virtual ~FMLLevelSet();

	CHAOS_API virtual Chaos::FImplicitObjectPtr CopyGeometry() const override;
	CHAOS_API virtual Chaos::FImplicitObjectPtr CopyGeometryWithScale(const FVec3& Scale) const override;

	CHAOS_API virtual FReal PhiWithNormal(const FVec3& x, FVec3& Normal) const override;
	CHAOS_API void BatchPhiWithNormal(const TConstArrayView<Softs::FPAndInvM> PAndInvMArray,
		const Softs::FSolverRigidTransform3& SolverToThis, TArray<Softs::FSolverReal>& OutBatchPhis, TArray<Softs::FSolverVec3>& OutBatchNormals,
		const Chaos::Softs::FSolverReal CollisionThickness, const int32 MLLevelsetThread, const int32 BatchBegin, const int32 BatchEnd) const;	

	CHAOS_API FReal SignedDistance(const FVec3& x) const;
	CHAOS_API void UpdateActiveBonesRelativeTransforms(TArray<FTransform>& InActiveBonesRelativeTransforms);
	CHAOS_API void UpdateActiveBonesRelativeTransformsAndUpdateDebugPhi(TArray<FTransform>& InActiveBonesRelativeTransforms);
	CHAOS_API void UpdateNeuralInferencesNumber(const int32 InNeuralInferencesNumber);

	const int32 GetNumberOfActiveBones() const { return ActiveBoneNames.Num(); }

	const TArray<FName>& GetActiveBoneNames() const { return ActiveBoneNames; }
	virtual const FAABB3 BoundingBox() const override { return LocalBoundingBox; } 
	const FVector3f GetTrainingGridMin() const { return TrainingGridMin; }
	const FVector3f GetTrainingGridVector(int32 Index) const { return TrainingGridUnitAxesXYZ[Index] * TrainingGridAxesLengthsXYZ[Index]; }
	
	// Returns a const ref to the underlying grid structure
	const TUniformGrid<FReal, 3>& GetGrid() const { return DebugGrid; }

	// Returns a const ref to the underlying phi grid
	const TArrayND<FReal, 3>& GetPhiArray() const { return DebugPhi; }

	FORCEINLINE static constexpr EImplicitObjectType StaticType()
	{
		return ImplicitObjectType::MLLevelSet;
	}

	FORCEINLINE void SerializeImp(FArchive& Ar)
	{
		FImplicitObject::SerializeImp(Ar);
		TBox<FReal, 3>::SerializeAsAABB(Ar, LocalBoundingBox);
		Ar << ActiveBoneNames;
		Ar << ActiveBonesRelativeTransforms;
		Ar << SignedDistanceModelWeights;
		Ar << SignedDistanceModelWeightsShapes;
		Ar << IncorrectZoneModelWeights;
		Ar << IncorrectZoneModelWeightsShapes;
		Ar << bUseIncorrectZoneModel;
		Ar << SignedDistanceScaling;
		Ar << TrainingGridMin;
		Ar << TrainingGridUnitAxesXYZ;
		Ar << TrainingGridAxesLengthsXYZ;
		Ar << TotalNumberOfRotationComponents;
		Ar << ActiveBonesRotationComponents;
		Ar << ActiveBonesReferenceRotations;
		Ar << ActiveBonesReferenceTranslations;
		Ar << DebugGrid; 
		Ar << DebugPhi;
		
		if (Ar.IsLoading())
		{
			SignedDistanceNeuralInferences.Empty();
			SignedDistanceNeuralInferences.Add(FMLLevelSetNeuralInference(NNESignedDistanceModel, SignedDistanceModelWeightsShapes));
			if (bUseIncorrectZoneModel)
			{
				IncorrectZoneNeuralInferences.Empty();
				IncorrectZoneNeuralInferences.Add(FMLLevelSetNeuralInference(NNEIncorrectZoneModel, IncorrectZoneModelWeightsShapes));
			}
		}
	}

	virtual void Serialize(FChaosArchive& Ar) override
	{
		SerializeImp(Ar);
	}

	virtual void Serialize(FArchive& Ar) override
	{
		SerializeImp(Ar);
	}	

	// Used to generate a simple debug surface
	CHAOS_API void GetZeroIsosurfaceGridCellFaces(TArray<FVector3f>& Vertices, TArray<FIntVector>& Tris) const;
	CHAOS_API void GetInteriorCells(TArray<TVec3<int32>>& InteriorCells, const FReal InteriorThreshold) const;
	CHAOS_API void CreatePhiFromMLModel();

	virtual uint32 GetTypeHash() const override
	{
		uint32 Result = 0;

		for(int32 LayerIdx = 0; LayerIdx < SignedDistanceModelWeights.Num(); ++LayerIdx)
		{
			for (int32 i = 0; i < SignedDistanceModelWeights[LayerIdx].Num(); ++i)
			{
				Result = HashCombine(Result, ::GetTypeHash(SignedDistanceModelWeights[LayerIdx][i]));
			}			
		}

		for (int32 LayerIdx = 0; LayerIdx < IncorrectZoneModelWeights.Num(); ++LayerIdx)
		{
			for (int32 i = 0; i < IncorrectZoneModelWeights[LayerIdx].Num(); ++i)
			{
				Result = HashCombine(Result, ::GetTypeHash(IncorrectZoneModelWeights[LayerIdx][i]));
			}
		}

		return Result;
	}

  private:
	void ProcessTrainingGridAxesVectors();
	void ComputeSignedDistanceNetworkWeightsInput(TArray<TArray<float, TAlignedHeapAllocator<64>>>& NetworkWeightsInput) const;
	void ComputeIncorrectZoneNetworkWeightsInput(TArray<TArray<float, TAlignedHeapAllocator<64>>>& NetworkWeightsInput) const;
	void ComputeWeightsInput(const TArray<FTransform>& RelativeBoneTransformationsInput, const TArray<TArray<float>>& NetworkWeights, const TArray<TArray<int32>>& NetworkWeightsShapes, TArray<TArray<float, TAlignedHeapAllocator<64>>>& NetworkWeightsInput) const;
	void LoadMLModelWeightsFromString(const FString& MLModelWeightsString, TArray<TArray<float>>& ModelWeightArray);
	void BuildNNEModel(const TArray<int32>& ModelArchitectureActivationNodeSizes, TObjectPtr<UNNEModelData> NNEModelData, const FString& ModelWeightsString,
		TSharedPtr<UE::NNE::IModelCPU>& NNEModel, TArray<FMLLevelSetNeuralInference>& NeuralInferences, TArray<TArray<int32>>& ModelWeightsShapes, TArray<TArray<float>>& ModelWeightArray);
	void BuildNNEModel(TObjectPtr<UNNEModelData> InNNEModelData, TSharedPtr<UE::NNE::IModelCPU>& NNEModel);
	const bool IsFTransformArraysDifferent(TArray<FTransform>& FTransformArr1, TArray<FTransform>& FTransformArr2, FReal Tol = UE_KINDA_SMALL_NUMBER) const;

	// We assume the closest active bone joint is always the first active bone. 
	const int32 GetClosestActiveBoneIndex(const FVec3 ParticlePositionMS) const { return 0; }  

  private:
	//Relative Bone Transform Related Data
	TArray<FName> ActiveBoneNames;
	TArray<FTransform> ActiveBonesRelativeTransforms;

	// NNE ML Model Elements
	// ToDo: Move these elements inside a new struct or FMLLevelSetNeuralInference
	TSharedPtr<UE::NNE::IModelCPU> NNESignedDistanceModel; 
	TArray<FMLLevelSetNeuralInference> SignedDistanceNeuralInferences;
	TArray<TArray<float>> SignedDistanceModelWeights; 
	TArray<TArray<int32>> SignedDistanceModelWeightsShapes;

	// NNE Incorrect Zone Model Elements
	// ToDo: Move these elements inside a new struct or FMLLevelSetNeuralInference
	TSharedPtr<UE::NNE::IModelCPU> NNEIncorrectZoneModel;
	TArray<FMLLevelSetNeuralInference> IncorrectZoneNeuralInferences;
	TArray<TArray<float>> IncorrectZoneModelWeights;
	TArray<TArray<int32>> IncorrectZoneModelWeightsShapes;
	bool bUseIncorrectZoneModel;
	
	// Inference Input Transformation. 
	FAABB3 LocalBoundingBox;
	float SignedDistanceScaling;
	FVector3f TrainingGridMin;
	TArray<FVector3f> TrainingGridUnitAxesXYZ;
	TArray<float> TrainingGridAxesLengthsXYZ;
	int32 TotalNumberOfRotationComponents;
	TArray<TArray<int32>> ActiveBonesRotationComponents;
	TArray<FVector3f> ActiveBonesReferenceRotations; 
	TArray<FVector3f>ActiveBonesReferenceTranslations; 

	//Debug Drawing
	TUniformGrid<FReal, 3> DebugGrid;
	TArrayND<FReal, 3> DebugPhi;

private:
	FMLLevelSet() : FImplicitObject(EImplicitObject::HasBoundingBox, ImplicitObjectType::MLLevelSet) {}	//needed for copy()
	CHAOS_API FMLLevelSet(TObjectPtr<UNNEModelData> InNNESignedDistanceModelData, TObjectPtr<UNNEModelData> InNNEIncorrectZoneModel); //needed for serialization

	friend FImplicitObject;	//needed for serialization 
	friend ::FKMLLevelSetElem; //needed for serialization 
};
}
