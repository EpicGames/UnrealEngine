// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Vector.h"
#include "ChaosCheck.h"
#include "Chaos/ArrayCollectionArray.h"
#include "Chaos/DynamicParticles.h"
#include "Chaos/UniformGrid.h"
#include "Algo/Unique.h"

namespace Chaos {

template<class T>
class TMPMTransfer
{
public:

	uint32 NPerSec = 2 * 2;
	uint32 NPerEle = 2 * 2 * 2;
	uint32 NTransfer = 3;
	int32 NumCells;

	//TODO(Yizhou): Think whether mpm transfer should just own the grid
	//(Is the grid used by the constraint as well?)
	TMPMGrid<T>& Grid;

	TArray<TVector<T, 3>> Weights;
	TArray<TVector<int32, 3>> Indices;
	TArray<TArray<int32>> CellData; //CellData[i] registers which particles are in ith cell
	//TArray<TArray<int32>> CellColors;

	//meta data for grid based cons
	TArray<TArray<int32>> ElementGridNodes;
	TArray<TArray<T>> ElementGridNodeWeights;
	TArray<TArray<TArray<int32>>> ElementGridNodeIncidentElements;
	TArray<TArray<int32>> ElementGridNodesSet;

	//APIC data;
	TArray<T> AArray;
	uint32 NumModes = 3;

	TMPMTransfer(){}
	TMPMTransfer(TMPMGrid<T>& _Grid) :Grid(_Grid) 
	{
		ensure(Grid.NPerDir == 2 || Grid.NPerDir == 3);
		NPerSec = Grid.NPerDir * Grid.NPerDir;
		NPerEle = NPerSec * Grid.NPerDir;
		//CellColors.SetNum(NPerEle);
	}
	TMPMTransfer(TMPMGrid<T>& _Grid, const int32 NumParticles) :Grid(_Grid)
	{
		ensure(Grid.NPerDir == 2 || Grid.NPerDir == 3);
		NPerSec = Grid.NPerDir * Grid.NPerDir;
		NPerEle = NPerSec * Grid.NPerDir;
		AArray.Init((T)0., 3 * 3 * NumParticles);
	}

	//template <typename ReorderFunctor, typename SplatFunctor>
	//This one does an initial splat of momentum and mass to the grid. 
	//One can add splat functor in the future passibly for other kinds of splat
	void InitialP2G(const TDynamicParticles<T, 3>& InParticles, TArray<T>& GridData) 
	{
		int32 N = InParticles.Size();

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(STAT_ChaosMPMTransferInitialBinning);
			Indices.SetNum(N);
			Weights.SetNum(N);

			/////////////////////////
			// compute weights and bin
			/////////////////////////

			//TODO(Yizhou): Do a timing and determine whether the following loop should 
			//use physics parallel for:
			for (int32 p = 0; p < N; p++)
			{
				Grid.BaseNodeIndex(InParticles.GetX(p), Indices[p], Weights[p]);
			}

		}
		/////////////////////
		// Computes which particles in the same cell
		/////////////////////
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(STAT_ChaosMPMTransferCellMetaCalc);
			NumCells = Grid.Size();
			CellData.SetNum(NumCells);
			for (int32 c = 0; c < NumCells; c++)
			{
				CellData[c].SetNum(0);
			}
			for (int32 p = 0; p < N; p++)
			{
				CellData[Grid.FlatIndex(Indices[p])].Emplace(p);
			}
		}

		/////////////////////
		// splat data to cells
		/////////////////////
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(STAT_ChaosMPMTransferSplatData);
			GridData.Init((T)0., CellData.Num() * (NTransfer + 1));

			TVector<int32, 3> GridCells = Grid.GetCells();
			
			for (int32 ii = 0; ii < int32(Grid.NPerDir); ii++)
			{
				for (int32 jj = 0; jj < int32(Grid.NPerDir); jj++)
				{
					for (int32 kk = 0; kk < int32(Grid.NPerDir); kk++)
					{
						int32 CurrentLocIndex = ii * NPerSec + jj * Grid.NPerDir + kk;
						PhysicsParallelFor(GridCells[0] / Grid.NPerDir, [&](const int32 iii)
							//{
						//for (int32 iii = 0; iii < int32(GridCells[0] / Grid.NPerDir); iii++)
						{
								for (int32 jjj = 0; jjj < int32(GridCells[1] / Grid.NPerDir); jjj++)
								{
									for (int32 kkk = 0; kkk < int32(GridCells[2] / Grid.NPerDir); kkk++)
									{
										TVector<int32, 3> MultiIndex = { iii * int32(Grid.NPerDir) + ii, jjj * int32(Grid.NPerDir) + jj, kkk * int32(Grid.NPerDir) + kk };
										int32 CellIndex = Grid.FlatIndex(MultiIndex);
										P2GApplyHelper(InParticles, CellIndex, GridData);
									}
								}
							});
						//}

					}
				}
			}

		}
	
	}

	//currently only splats mass and momentum
	void P2GApplyHelper(const TDynamicParticles<T, 3>& InParticles, const int32 CellIndex, TArray<T>& GridData)
	{
		//check if valid cell:
		if (CellIndex < NumCells && CellData[CellIndex].Num() > 0)
		{
			for (int32 i = 0; i < CellData[CellIndex].Num(); i++)
			{
				int32 p = CellData[CellIndex][i];
				for (int iii = 0; iii < int32(Grid.NPerDir); iii++)
				{
					T Nii = Grid.Nijk(Weights[p][0], iii);
					for (int jjj = 0; jjj < int32(Grid.NPerDir); jjj++)
					{
						T Njj = Grid.Nijk(Weights[p][1], jjj);
						for (int kkk = 0; kkk < int32(Grid.NPerDir); kkk++)
						{
							TVector<int32, 3> LocIndex = { iii, jjj, kkk };
							TVector<int32, 3> GlobMultiIndex = Grid.Loc2GlobIndex(Indices[p], LocIndex);
							int32 GlobIndex = Grid.FlatIndex(GlobMultiIndex);
							T Nkk = Grid.Nijk(Weights[p][2], kkk);
							T NProd = Nii * Njj * Nkk;
							GridData[(NTransfer + 1) * GlobIndex] += NProd * InParticles.M(p);
							for (int32 alpha = 0; alpha < 3; alpha++)
							{
								GridData[(NTransfer + 1) * GlobIndex + alpha + 1] += NProd * InParticles.M(p) * InParticles.V(p)[alpha];
							}
							APICP2G(p, GlobIndex, NProd * InParticles.M(p), InParticles, GridData);
						}
					}
				}
			}
		}
	}

	void APICP2G(const int32 p, const int32 GlobIndex, const T mip, const TDynamicParticles<T, 3>& Particles, TArray<T>& GridData)
	{
		TVector<T, 3> xi_minus_xp = Grid.Node(GlobIndex) - Particles.GetX(p);
		for (int32 l = 0; l < 3; ++l) 
		{
			T vl = T(0);
			for (int32 m = 0; m < int32(NumModes); ++m)
				vl += AArray[NumModes * 3 * p + NumModes * l + m] * xi_minus_xp[m];

			GridData[l + (NTransfer + 1) * GlobIndex + 1] += mip * vl;
		}
	
	}

	void ComputeElementMetaData(const TArray<TVector<int32, 4>>& InMesh)
	{
		ElementGridNodes.SetNum(InMesh.Num());
		ElementGridNodeWeights.SetNum(InMesh.Num());
		ElementGridNodeIncidentElements.SetNum(InMesh.Num());
		ElementGridNodesSet.SetNum(InMesh.Num());
		//TODO(Yizhou): make following loop parallel for with appropriate bool condition
		for (int32 e = 0; e < InMesh.Num(); e++)
		{
			ElementGridNodes[e].SetNum(NPerEle * 4);
			ElementGridNodeWeights[e].SetNum(NPerEle * 4);
			for (int32 ie = 0; ie < 4; ie++)
			{
				int32 p = InMesh[e][ie];
				TVector<int32, 3> Index = Indices[p];
				for (int32 ii = 0; ii < int32(Grid.NPerDir); ii++)
				{
					T Nii = Grid.Nijk(Weights[p][0], ii);
					for (int32 jj = 0; jj < int32(Grid.NPerDir); jj++)
					{
						T Njj = Grid.Nijk(Weights[p][1], jj);
						for (int32 kk = 0; kk < int32(Grid.NPerDir); kk++)
						{
							T Nkk = Grid.Nijk(Weights[p][2], kk);
							TVector<int32, 3> LocIndex = { ii, jj, kk };
							TVector<int32, 3> GlobIndex = Grid.Loc2GlobIndex(Index, LocIndex);
							int32 GlobIndexFlat = Grid.FlatIndex(GlobIndex);
							ElementGridNodes[e][ie * NPerEle + ii * NPerSec + jj * Grid.NPerDir + kk] = GlobIndexFlat;
							ElementGridNodeWeights[e][ie * NPerEle + ii * NPerSec + jj * Grid.NPerDir + kk] = Nii * Njj * Nkk;
						}
					}
				}
			}
			ComputeIncidentElements(ElementGridNodes[e], ElementGridNodeIncidentElements[e]);
			ElementGridNodesSet[e].SetNum(ElementGridNodeIncidentElements[e].Num());
			for (int32 kk = 0; kk < ElementGridNodeIncidentElements[e].Num(); kk++)
			{
				ElementGridNodesSet[e][kk] = ElementGridNodes[e][ElementGridNodeIncidentElements[e][kk][0]];
			}
		}
	}

	//computes incident elements in serial
	static void ComputeIncidentElements(const TArray<int32>& ArrayIn, TArray<TArray<int32>>& IncidentElements) 
	{
		TArray<int32> Ordering, Ranges;
		Ordering.SetNum(ArrayIn.Num());
		Ranges.SetNum(ArrayIn.Num()+1);

		for (int32 i = 0; i < ArrayIn.Num(); ++i) 
		{
			Ordering[i] = i;
			Ranges[i] = i;
		}
		
		//TODO(Yizhou): decide to use sort or heap sort or merge sort. 
		Ordering.Sort([&ArrayIn](const int32 A, const int32 B) 
		{
		return ArrayIn[A] < ArrayIn[B];
		});

		int32 Last = Algo::Unique(Ranges, [&ArrayIn, &Ordering](int32 i, int32 j) { return ArrayIn[Ordering[i]] == ArrayIn[Ordering[j]]; });

		int32 NumNodes = Last - 1;
		Ranges[NumNodes] = ArrayIn.Num();
		Ranges.SetNum(NumNodes + 1);

		IncidentElements.SetNum(Ranges.Num() - 1);

		for (int32 p = 0; p < IncidentElements.Num(); ++p) 
		{
			IncidentElements[p].SetNum(Ranges[p + 1] - Ranges[p]);
			for (int32 e = Ranges[p]; e < Ranges[p + 1]; e++) 
			{
				IncidentElements[p][e - Ranges[p]] = Ordering[e];
			}
		}
	}

	void ComputeGridPositions(const TArray<T>& GridData, const T Dt, TArray<TVector<T, 3>>& GridPositions)
	{
		GridPositions.SetNum(Grid.Size());
		for (int32 i = 0; i < Grid.Size(); i++)
		{
			TVector<T,3> XOld = Grid.Node(i);
			T Mass = GridData[(NTransfer + 1) * i];
			if (Mass == 0) {
				GridPositions[i] = XOld;
			}
			else {
				for (int32 Alpha = 0; Alpha < int32(NTransfer); Alpha++)
					GridPositions[i][Alpha] = Dt * GridData[(NTransfer + 1) * i + Alpha + 1] / Mass + XOld[Alpha];
			}
		}
	}


	TVec4<TVector<T, 3>> SparseG2P(const TArray<TVector<T, 3>>& GridPositions, const int32 ElementIndex)
	{
		TVec4<TVector<T, 3>> Result(TVector<T, 3>((T)0.));

		for (int32 i = 0; i < 4; i++) {
			for (int32 ii = 0; ii < int32(NPerEle); ii++) {
				int32 FlatIndex = ElementGridNodes[ElementIndex][NPerEle * i + ii];
				for (int32 Alpha = 0; Alpha < 3; Alpha++) {
					Result[i][Alpha] += ElementGridNodeWeights[ElementIndex][NPerEle * i + ii] * GridPositions[FlatIndex][Alpha];
				}
			}
		}
		
		return Result;
	}

	void ComputeAArray(const TArray<T>& GridData, const TDynamicParticles<T, 3>& Particles)
	{
		AArray.Init((T)0., NumModes * 3 * Particles.Size());
		PhysicsParallelFor(int32(Particles.Size()), [&](const int32 p)
			//for (int32 p = 0; p < int32(Particles.Size()); p++)
			{
				TVector<int32, 3> BaseIndex = Indices[p];
				for (int32 ii = 0; ii < int32(Grid.NPerDir); ii++)
				{
					T Nii = Grid.Nijk(Weights[p][0], ii);
					T dNii = Grid.dNijk(Weights[p][0], ii, Grid.GetDx()[0]);
					for (int32 jj = 0; jj < int32(Grid.NPerDir); jj++)
					{
						T Njj = Grid.Nijk(Weights[p][1], jj);
						T dNjj = Grid.dNijk(Weights[p][1], jj, Grid.GetDx()[0]);
						for (int32 kk = 0; kk < int32(Grid.NPerDir); kk++)
						{
							T Nkk = Grid.Nijk(Weights[p][2], kk);
							T dNkk = Grid.dNijk(Weights[p][2], kk, Grid.GetDx()[0]);
							TVector<T, 3> Ni = { Nii, Njj, Nkk };
							TVector<T, 3> dNi = { dNii, dNjj, dNkk };
							TVector<int32, 3> LocIndex = { ii, jj, kk };
							TVector<int32, 3> GlobIndex = Grid.Loc2GlobIndex(BaseIndex, LocIndex);
							int32 GlobIndexFlat = Grid.FlatIndex(GlobIndex);
							if (GridData[(NTransfer + 1) * GlobIndexFlat] != T(0))
							{
								if (Grid.interp == TMPMGrid<T>::linear)
								{
									TVector<T, 3> dN((T)0.);
									Grid.GradNi(Ni, dNi, dN);
									for (int32 l = 0; l < 3; l++)
									{
										T GridQuantity = GridData[(NTransfer + 1) * GlobIndexFlat + 1 + l] / GridData[(NTransfer + 1) * GlobIndexFlat];
										for (int32 iii = 0; iii < int32(NumModes); ++iii)
											AArray[NumModes * 3 * p + NumModes * l + iii] += dN[iii] * GridQuantity;
									}
								}
								else
								{
									TVector<T, 3> rip = Grid.Node(GlobIndexFlat) - Particles.GetX(p);
									T NiProdbInv = (T(4) * Ni[0] * Ni[1] * Ni[2]) / (Grid.GetDx()[0] * Grid.GetDx()[0]);
									for (int32 l = 0; l < 3; l++)
									{
										T GridQuantity = GridData[(NTransfer + 1) * GlobIndexFlat + 1 + l] / GridData[(NTransfer + 1) * GlobIndexFlat];
										for (int32 iii = 0; iii < int32(NumModes); ++iii)
											AArray[NumModes * 3 * p + NumModes * l + iii] += NiProdbInv * rip[iii] * GridQuantity;
									}
								}
							}

						}
					}
				}
			}, Particles.Size() < 50);
	}

	void GridPositionsToGridData(const TArray<TVector<T, 3>>& GridPositions, const T Dt, TArray<T>& GridData)
	{
		for (int32 i = 0; i < GridPositions.Num(); i++)
		{
			T Mi = GridData[(NTransfer + 1) * i];
			if (Mi != T(0))
			{
				TVector<T, 3> NodePos = Grid.Node(i);
				for (int32 alpha = 0; alpha < 3; alpha++)
				{
					GridData[(NTransfer + 1) * i + alpha + 1] = Mi * (GridPositions[i][alpha] - NodePos[alpha]) / Dt;
				}
			}
			//else 
			//{
			//	for (int32 alpha = 0; alpha < 3; alpha++)
			//	{
			//	}
			//}
		}
	
	}

	void G2P(const TArray<T>& GridData, TDynamicParticles<T, 3>& Particles)
	{
		PhysicsParallelFor(Particles.Size(), [&](const int32 p)
			{
				Particles.V(p) = TVec3<T>((T)0.);
				for (int32 ii = 0; ii < int32(Grid.NPerDir); ii++)
				{
					T Nii = Grid.Nijk(Weights[p][0], ii);
					for (int32 jj = 0; jj < int32(Grid.NPerDir); jj++)
					{
						T Njj = Grid.Nijk(Weights[p][1], jj);
						for (int32 kk = 0; kk < int32(Grid.NPerDir); kk++)
						{
							T Nkk = Grid.Nijk(Weights[p][2], kk);
							TVector<int32, 3> LocIndex = { ii, jj, kk };
							TVector<int32, 3> GlobIndex = Grid.Loc2GlobIndex(Indices[p], LocIndex);
							int32 GlobIndexFlat = Grid.FlatIndex(GlobIndex);
							if (GridData[GlobIndexFlat * (NTransfer + 1)] > T(0))
							{
								for (int32 alpha = 0; alpha < 3; alpha++)
								{
									Particles.V(p)[alpha] += Nii * Njj * Nkk * GridData[GlobIndexFlat * (NTransfer + 1) + alpha + 1] / GridData[GlobIndexFlat * (NTransfer + 1)];
								}
							}
						}
					}
				}
			}, Particles.Size() < 50);
	}


	TArray<TVector<T, 3>> SparseP2G(const TVec4<TVector<T, 3>>& InGradient, const int32 ElementIndex)
	{
		
		TArray<TVector<T, 3>> GridGradient;
		GridGradient.Init(TVector<T, 3>((T)0.), 4 * NPerEle);
		for (int32 i = 0; i < 4; i++) {
			for (int32 ii = 0; ii < int32(NPerEle); ii++) {
				for (int32 alpha = 0; alpha < 3; alpha++) {
					GridGradient[NPerEle * i + ii][alpha] += ElementGridNodeWeights[ElementIndex][NPerEle * i + ii] * InGradient[i][alpha];
				}
			}
		}

		TArray<TVector<T, 3>> GridGradientCompact;
		GridGradientCompact.Init(TVector<T, 3>((T)0.), ElementGridNodeIncidentElements[ElementIndex].Num());

		for (int32 node = 0; node < ElementGridNodeIncidentElements[ElementIndex].Num(); node++) 
		{
			for (int32 node_indices = 0; node_indices < ElementGridNodeIncidentElements[ElementIndex][node].Num(); node_indices++) 
			{
				for (int32 alpha = 0; alpha < 3; alpha++) 
				{
					GridGradientCompact[node][alpha] += GridGradient[ElementGridNodeIncidentElements[ElementIndex][node][node_indices]][alpha];
				}
			}
		}

		return GridGradient;
	
	}

};

}  
