// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Defines.h"

#if !COMPILE_WITHOUT_UNREAL_SUPPORT
#include "Containers/Map.h"
#else
#include <unordered_map>

template<class Key, class Value>
class TMap : public std::unordered_map<Key, Value>
{
  public:
	Value& FindOrAdd(const Key& Index) { return operator[](Index); }
};
#endif
