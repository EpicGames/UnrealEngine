// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/ParticleHandleFwd.h"

#include "Math/UnrealMathUtility.h"

namespace Chaos
{
	struct FInertiaConditioningTolerances
	{
		float InvMassTolerance = 1.e-20f;
		float InvInertiaTolerance = 1.e-4f;
		float ExtentTolerance = 1.e-8f;
	};


	/**
	 * Calculate an inertia scale so that rotation corrections when applying a constraint are no larger than some fraction of the total correction.
	 * It has the net effect of making the object "more round" and also increasing the inertia relative to the mass when the object is not of "regular" proportions.
	 * 
	 * @param InvM the inverse mass
	 * @param InvI the inverse inertia
	 * @param ConstraintExtents the Maximum constraint arm distance along each axis. This should be the extents of the object, including all shapes and joint connectors.
	 * @param MaxDistance the constraint error that we want to be stable. Corrections above this may still contain large rotation components.
	 * @param MaxRotationRatio the contribution to the constraint correction from rotation of the object will be less that this fraction of the total error.
	 * @param MaxInvInertiaComponentRatio the maximum allowed ratio between the smallest and largest components of the inverse inertia.
	 * @param Tolerances the tolerances to use
	*/
	FVec3f CHAOS_API CalculateInertiaConditioning(const FRealSingle InvM, const FVec3f& InvI, const FVec3f& ConstraintExtents, const FRealSingle MaxDistance, const FRealSingle MaxRotationRatio, const FRealSingle MaxInvInertiaComponentRatio, const FInertiaConditioningTolerances& Tolerances = FInertiaConditioningTolerances());

	/**
	 * Calculate an inertia scale for a particle based on object size and all joint connector offsets that should stabilize the constraints.
	 * @see CalculateInertiaConditioning()
	 * @param Rigid the particle to calculate inertia conditioning for
	 * @param MaxDistance the constraint error that we want to be stable. Corrections above this may still contain large rotation components.
	 * @param MaxRotationRatio the contribution to the constraint correction from rotation of the object will be less that this fraction of the total error.
	 * @param MaxInvInertiaComponentRatio the maximum allowed ratio between the smallest and largest components of the inverse inertia.
	*/
	FVec3f CHAOS_API CalculateParticleInertiaConditioning(const FPBDRigidParticleHandle* Rigid, const FRealSingle MaxDistance, const FRealSingle MaxRotationRatio, const FRealSingle MaxInvInertiaComponentRatio, const FInertiaConditioningTolerances& Tolerances = FInertiaConditioningTolerances());

}