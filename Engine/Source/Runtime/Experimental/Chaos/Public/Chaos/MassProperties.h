// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/Defines.h"
#include "Chaos/Matrix.h"
#include "Chaos/Rotation.h"
#include "Chaos/Vector.h"
#include "Containers/ArrayView.h"

#if !COMPILE_WITHOUT_UNREAL_SUPPORT
namespace Chaos
{
	class FTriangleMesh;
	class FPerShapeData;
	class FImplicitObject;
	class FImplicitObjectUnion;

	template<class T, int d>
	class TParticles;
	using FParticles = TParticles<FReal, 3>;

	struct FMassProperties
	{
		FMassProperties()
			: Mass(0)
			, Volume(0)
			, CenterOfMass(0)
			, RotationOfMass(FRotation3::FromElements(FVec3(0), 1))
			, InertiaTensor(0)
		{}
		FReal Mass;
		FReal Volume;
		FVec3 CenterOfMass;
		FRotation3 RotationOfMass;
		FMatrix33 InertiaTensor;
	};

	template <typename T>
	TRotation<T,3> CHAOS_API TransformToLocalSpace(PMatrix<T,3,3>& Inertia);
	void CHAOS_API TransformToLocalSpace(FMassProperties& MassProperties);

	template<typename T, typename TSurfaces>
	void CHAOS_API CalculateVolumeAndCenterOfMass(const TParticles<T,3>& Vertices, const TSurfaces& Surfaces, T& OutVolume, TVec3<T>& OutCenterOfMass);

	template<typename T, typename TSurfaces>
	void CHAOS_API CalculateVolumeAndCenterOfMass(const TArray<TVec3<T>>& Vertices, const TSurfaces& Surfaces, T& OutVolume, TVec3<T>& OutCenterOfMass);

	template<typename T, typename TSurfaces>
	void CHAOS_API CalculateVolumeAndCenterOfMass(const TArray<UE::Math::TVector<T>>& Vertices, const TSurfaces& Surfaces, T& OutVolume, UE::Math::TVector<T>& OutCenterOfMass);
	
	template<typename TSurfaces>
	FMassProperties CHAOS_API CalculateMassProperties(const FParticles& Vertices, const TSurfaces& Surfaces, const FReal Mass);

	template<typename T, typename TSurfaces>
	void CHAOS_API CalculateInertiaAndRotationOfMass(const TParticles<T,3>& Vertices, const TSurfaces& Surfaces, const T Density, const TVec3<T>& CenterOfMass,
		PMatrix<T,3,3>& OutInertiaTensor, TRotation<T, 3>& OutRotationOfMass);

	template<typename T, typename TSurfaces>
	void CHAOS_API CalculateInertiaAndRotationOfMass(const TArray<UE::Math::TVector<T>>& Vertices, const TSurfaces& Surfaces, const T Density, const UE::Math::TVector<T>& CenterOfMass,
		PMatrix<T,3,3>& OutInertiaTensor, TRotation<T, 3>& OutRotationOfMass);
	
	void CHAOS_API CalculateVolumeAndCenterOfMass(const FBox& BoundingBox, FVector::FReal& OutVolume, FVector& OutCenterOfMass);
	void CHAOS_API CalculateInertiaAndRotationOfMass(const FBox& BoundingBox, const FVector::FReal Density, FMatrix33& OutInertiaTensor, FRotation3& OutRotationOfMass);

	// Combine a list of transformed inertia tensors into a single inertia. Also diagonalize the inertia and set the rotation of mass accordingly. 
	// This is equivalent to a call to CombineWorldSpace followed by TransformToLocalSpace.
	// @see CombineWorldSpace()
	FMassProperties CHAOS_API Combine(const TArray<FMassProperties>& MPArray);

	// Combine a list of transformed inertia tensors into a single inertia tensor.
	// @note The inertia matrix is not diagonalized, and any rotation will be built into the matrix (RotationOfMass will always be Identity)
	// @see Combine()
	FMassProperties CHAOS_API CombineWorldSpace(const TArray<FMassProperties>& MPArray);

	// Calculate the mass properties from a union
	bool CHAOS_API CalculateMassPropertiesOfImplicitUnion(
	Chaos::FMassProperties& OutMassProperties,
	const Chaos::FRigidTransform3& WorldTransform,
	const Chaos::FImplicitObjectUnion& ImplicitUnion,
	Chaos::FReal InDensityKGPerCM);
	
	// Calculate the mass properties from a specific implicit
	bool CHAOS_API CalculateMassPropertiesOfImplicitType(
		FMassProperties& OutMassProperties,
		const FRigidTransform3& WorldTransform,
		const FImplicitObject* ImplicitObject,
		Chaos::FReal InDensityKGPerCM);
	
	// Calculate the mass properties from a list of shapes
	void CHAOS_API CalculateMassPropertiesFromShapeCollection(
		FMassProperties& OutProperties, 
		int32 InNumShapes, 
		FReal InDensityKGPerCM,
		const TArray<bool>& bContributesToMass,
		TFunction<FPerShapeData* (int32 ShapeIndex)> GetShapeDelegate);

	template<typename T>
	T KgCm3ToKgM3(T Density)
	{
		return Density * (T)1000000;
	}

	template<typename T>
	T KgM3ToKgCm3(T Density)
	{
		return Density / (T)1000000;
	}

	template<typename T>
	T GCm3ToKgCm3(T Density)
	{
		return Density / (T)1000;
	}

	template<typename T>
	T KgCm3ToGCm3(T Density)
	{
		return Density * (T)1000;
	}

	template <typename T, int d>
	using TMassProperties UE_DEPRECATED(4.27, "Deprecated. this class is to be deleted, use FMassProperties instead") = FMassProperties;
}

#endif
