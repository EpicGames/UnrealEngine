// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Chaos/Core.h"
#include "Templates/PimplPtr.h"

namespace Chaos {

// Note: currently only explicitly instantiated for 3x3 blocks
template<typename T, int32 BlockSize>
class TBlockSparseSymmetricLinearSystem final
{
public:
	CHAOS_API TBlockSparseSymmetricLinearSystem();
	TBlockSparseSymmetricLinearSystem(const TBlockSparseSymmetricLinearSystem&) = delete;
	CHAOS_API TBlockSparseSymmetricLinearSystem(TBlockSparseSymmetricLinearSystem&&);
	~TBlockSparseSymmetricLinearSystem() = default;
	TBlockSparseSymmetricLinearSystem& operator=(TBlockSparseSymmetricLinearSystem&) = delete;
	CHAOS_API TBlockSparseSymmetricLinearSystem& operator=(TBlockSparseSymmetricLinearSystem&&);

	// Square system so NumRows == NumCols
	void Reset(int32 NumRows);
	void ReserveForParallelAdd(int32 NumDiagEntries, int32 NumOffDiagEntries);

	void AddMatrixEntry(int32 Index0, int32 Index1, const Chaos::PMatrix<T, BlockSize, BlockSize>& AEntry);

	// Call after all Matrix Entries have been added
	void FinalizeSystem();

	bool Solve(const TConstArrayView<TVector<T, BlockSize>>& RHS, const TArrayView<TVector<T, BlockSize>>& Result,
		const int32 MaxNumCGIterations,	const T CGResidualTolerance, bool bCheckResidual, 
		int32* OptionalOutIterations = nullptr, T* OptionalOutError = nullptr) const;

private:

	// Putting everything in here to avoid exposing Eigen
	struct FPimpl;
	TPimplPtr<FPimpl> Pimpl;
};

} // namespace Chaos

