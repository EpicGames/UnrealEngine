// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ChaosCheck.h"
#include "ChaosLog.h"
#include "Chaos/Framework/Parallel.h"
#include "GenericPlatform/GenericPlatformMath.h"
#include "Misc/AssertionMacros.h"
#include "Chaos/Vector.h"

namespace Chaos {

template <typename Func, class T>
void LanczosCG(
	Func multiplyA, 
	TArray<T>& x,
	const TArray<T>& b,
	const int max_it, 
	const T res = 1e-4, 
	bool check_residual = false,
	int min_parallel_batch_size = 1000) 
{
	//ToDo(ChaosFlesh): take an optional use_list of vertex indices to bypass unused verts
	auto dotProduct = [](const TArray<T>& x, const TArray<T>& y) 
	{
		T result = T(0);
		checkfSlow(x.Num() == y.Num(), TEXT("LanczosCG: trying to take the dot product with vectors of different size."));
		//PhysicsParallelFor(x.Num(), [&](const int32 i)
		for (int32 i = 0; i < x.Num(); i++)
		{
			result += x[i] * y[i];
		}/*,
		x.Num() < min_parallel_batch_size);*/
		return result;
	};

	auto AXPY = [min_parallel_batch_size](TArray<T>& y, T a, const TArray<T>& x) 
	{
		checkfSlow(x.Num() == y.Num(), TEXT("LanczosCG: trying to take a linear combination of vectors with different sizes."));
		PhysicsParallelFor(x.Num(), [&](const int32 i)
		{
			y[i] += a * x[i];
	    },
		x.Num() < min_parallel_batch_size); // Single threaded for less than min_parallel_batch_size operations.
	};

	auto scale = [min_parallel_batch_size](TArray<T>& y, T a)
	{
		PhysicsParallelFor(y.Num(), [&](const int32 i)
		{
			y[i] *= a;
	    },
		y.Num() < min_parallel_batch_size); // Single threaded for less than min_parallel_batch_size operations.
	};

	auto set = [min_parallel_batch_size](TArray<T>& y, const TArray<T>& x)
	{
		checkfSlow(x.Num() == y.Num(), TEXT("LanczosCG: trying to set to vectors with different sizes."));
		PhysicsParallelFor(x.Num(), [&](const int32 i)
		{
			y[i] = x[i];
		},
		x.Num() < min_parallel_batch_size); // Single threaded for less than 1k operations.
	};

	LanczosCG<T>(multiplyA, dotProduct, AXPY, scale, set, x, b, max_it, res, check_residual);
}

template <class T, typename Func, int32 d=3>
void LanczosCG(
	Func multiplyA, 
	TArray<TVector<T, d>>& x,
	const TArray<TVector<T, d>>& b,
	const int max_it, 
	const T res = 1e-4, 
	const TArray<int32>* use_list = nullptr)
{
	using TV = TVector<T, d>;
	auto dot_product = [&use_list](const TArray<TV>& x, const TArray<TV>& y)
	{
		checkfSlow(x.Num() == y.Num(), TEXT("Chaos::LanczosCG()::dot_product[]: x and y not sized consistently."));
		T result = T(0);
		if (use_list != nullptr) 
		{
			//PhysicsParallelFor(use_list->Num(), [&](const int32 i)
			//{
				for (int32 i = 0; i < use_list->Num(); ++i) 
				{
					T dot = T(0);
					int32 j = (*use_list)[i];
					for (int32 alpha = 0; alpha < d; alpha++) 
					{
						dot += x[j][alpha] * y[j][alpha];
					}
					result += dot;
				}
			//},
			//use_list->Num() < 1000);
		}
		else 
		{
			//PhysicsParallelFor(x.Num(), [&](const int32 i)
			//{
			for (int32 i = 0; i < x.Num(); i++)
			{
				T dot = T(0);
				for (size_t alpha = 0; alpha < d; alpha++) 
				{
					dot += x[i][alpha] * y[i][alpha];
				}
				result += dot;
			}/*,
			x.Num() < 1000);*/
		}
		return result;
	};
	auto AXPY = [&use_list](TArray<TV>& y, T a, const TArray<TV>& x) 
	{
		checkfSlow(x.Num() == y.Num(), TEXT("Chaos::LanczosCG()::AXPY[]: x and y not sized consistently."));
		if (use_list != nullptr) 
		{
			PhysicsParallelFor(use_list->Num(), [&](const int32 i) 
			{
				const int32 j = (*use_list)[i];
				for (int32 alpha = 0; alpha < d; alpha++) 
				{
					y[j][alpha] += a * x[j][alpha];
				}
			},
			use_list->Num() < 1000);
		}
		else 
		{
			PhysicsParallelFor(x.Num(), [&](const int32 i)
			{
				for (int32 alpha = 0; alpha < d; alpha++) 
				{
					y[i][alpha] += a * x[i][alpha];
				}
			},
			x.Num() < 1000);
		}
	};
	auto scale = [&use_list](TArray<TV>& y, T a) 
	{
		if (use_list != nullptr) 
		{
			PhysicsParallelFor(use_list->Num(), [&](const int32 i)
			{
				const int32 j = (*use_list)[i];
				for (int32 alpha = 0; alpha < d; alpha++) 
				{
					y[j][alpha] = a * y[j][alpha];
				}
			});
		}
		else 
		{
			PhysicsParallelFor(y.Num(), [&](const int32 i)
			{
				for (size_t alpha = 0; alpha < d; alpha++) 
				{
					y[i][alpha] = a * y[i][alpha];
				}
			});
		}
	};
	auto set = [&use_list](TArray<TV>& y, const TArray<TV>& x) 
	{
		if (use_list != nullptr) 
		{
			PhysicsParallelFor(use_list->Num(), [&](const int32 i)
			{
				const int32 j = (*use_list)[i];
				for (size_t alpha = 0; alpha < d; alpha++) 
				{
					y[j][alpha] = x[j][alpha];
				}
			});
		}
		else 
		{
			PhysicsParallelFor(y.Num(), [&](const int32 i)
			{
				for (int32 alpha = 0; alpha < d; alpha++) 
				{
					y[i][alpha] = x[i][alpha];
				}
			});
		}
	};

	LanczosCG<T>(multiplyA, dot_product, AXPY, scale, set, x, b, max_it, res);
}

template <typename T, typename TV, typename Func1, typename Func2, typename Func3, typename Func4, typename Func5>
void LanczosCG(
	Func1 multiplyA, 
	Func2 dotProduct, 
	Func3 AXPY, 
	Func4 scale, 
	Func5 set, 
	TArray<TV>& x,
	const TArray<TV>& b,
	const int max_it, 
	const T res = T(1e-4), 
	bool check_residual = false) 
{
	// multiplyA = [&A](TV &y, const TV& x): y = A*x;
	// dotProduct = [](const TV& x, const TV& y): return x^Ty
	// AXPY = [](Vector& y, T a, const TV& x): y += a*x;
	// scale = [](Vector& y, T a): y <- a*y
	// set = [](TV& y, const TV& x): y <- x
	// Printing out: use
	// (*((TV*)&b))[i]
	// ((Vector*)&b)->operator[](i)

	TArray<TV> v; v.SetNum(x.Num());
	TArray<TV> q; q.SetNum(x.Num());
	TArray<TV> q_old; q_old.SetNum(x.Num());
	TArray<TV> c; c.SetNum(x.Num());

	T beta = FGenericPlatformMath::Sqrt(T(dotProduct(b, b)));
	if (beta <= res) 
	{
		UE_LOGF(LogChaos, Warning, "Lanczos residual = %f. Lanczos converged in 0 iterations.", beta)
		scale(x, T(0));
		return;
	}
	set(q, b);
	scale(q, T(1) / beta);
	multiplyA(v, q);
	T alpha = dotProduct(v, q);
	T d = alpha;
	T p = beta / d;
	set(c, q);
	set(x, c);
	scale(x, p);
	AXPY(v, -alpha, q);
	T residual{};
	TArray<TV> y; y.SetNum(x.Num());
	for (int it = 1; it < max_it; it++) 
	{
		beta = FGenericPlatformMath::Sqrt(T(dotProduct(v, v)));
		if (check_residual == true)
		{
			multiplyA(y, x);
			AXPY(y, T(-1), b);
			residual = FGenericPlatformMath::Sqrt(T(dotProduct(y, y)));
			if (residual < res) 
			{
				UE_LOGF(LogChaos, VeryVerbose, "Ax - b residual = %g. Lanczos converged in %d iterations.", residual, it);
				return;
			}
		}
		else
		{
			if (beta < res) 
			{
				UE_LOGF(LogChaos, VeryVerbose, "Lanczos residual = %g. Lanczos converged in %d iterations.", beta, it);
				return;
			}
		}
		T mu = beta / d;
		set(q_old, q);
		set(q, v);
		scale(q, T(1) / beta);
		multiplyA(v, q);
		alpha = dotProduct(v, q);
		T d_new = alpha - mu * mu * d;
		p = -p * d * mu / d_new;
		scale(c, -mu);
		AXPY(c, T(1), q);
		AXPY(x, p, c);
		AXPY(v, -alpha, q);
		AXPY(v, -beta, q_old);
		d = d_new;
	}

	if (check_residual)
	{
		UE_LOGF(LogChaos, Verbose, "Lanczos used max iterations (%d). Residual = %g.", max_it, residual);
	}
	else
	{
		UE_LOGF(LogChaos, Verbose, "Lanczos used max iterations (%d).", max_it);
	}
}


} // namespace Chaos

