// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

//#include "ChaosFlesh/FleshCollectionUtility.h"
#include "Chaos/Framework/Parallel.h"
#include "Chaos/Math/Krylov.h"
#include "Chaos/UniformGrid.h"
#include "Chaos/Vector.h"
#include "Math/Matrix.h"
#include "Math/Vector.h"
#include "Chaos/Utilities.h"
#include <iostream>

namespace Chaos {

// Set/Get i, j

template <class T>
inline void RowMaj3x3Set(T* A, const int32 i, const int32 j, const T Value)
{
	A[3 * i + j] = Value;
}

template <class T>
inline const T& RowMaj3x3Get(const T* A, const int32 i, const int32 j)
{
	return A[3 * i + j];
}

template <class T>
inline T& RowMaj3x3Get(T* A, const int32 i, const int32 j)
{
	return A[3 * i + j];
}

// Set/Get row

template <class T>
inline void RowMaj3x3SetRow(T* A, const int32 i, const T* Values)
{
	for (int32 j = 0; j < 3; j++)
	{
		RowMaj3x3Set(A, i, j, Values[i]);
	}
}

template <class T, class TV>
inline void RowMaj3x3SetRow(T* A, const int32 i, const TV Value)
{
	const T Tmp[3]{ static_cast<T>(Value[0]), static_cast<T>(Value[1]), static_cast<T>(Value[2]) };
	RowMaj3x3SetRow(A, i, &Tmp[0]);
}

template <class T, class TV>
inline void RowMaj3x3GetRow(T* A, const int32 i, TV& Row)
{
	for (int j = 0; j < 3; j++)
	{
		Row[j] = RowMaj3x3Get(A, i, j);
	}
}

// Set/Get col

template <class T>
inline void RowMaj3x3SetCol(T* A, const int32 j, const T* Values)
{
	for (int32 i = 0; i < 3; i++)
	{
		RowMaj3x3Set(A, i, j, Values[i]);
	}
}

template <class T, class TV>
inline void RowMaj3x3SetCol(T* A, const int32 j, const TV Value)
{
	for (int32 i = 0; i < 3; i++)
	{
		RowMaj3x3Set(A, i, j, static_cast<T>(Value[i]));
	}
}

template <class T, class TV>
inline void RowMaj3x3GetCol(T* A, const int32 j, TV& Col)
{
	for (int i = 0; i < 3; i++)
	{
		Col[i] = RowMaj3x3Get(A, i, j);
	}
}

// Determinant

template <class T>
inline T RowMaj3x3Determinant(const T A0, const T A1, const T A2, const T A3, const T A4, const T A5, const T A6, const T A7, const T A8)
{
	return A0 * (A4 * A8 - A5 * A7) - A1 * (A3 * A8 - A5 * A6) + A2 * (A3 * A7 - A4 * A6);
}

template <class T>
inline T RowMaj3x3Determinant(const T* A)
{
	return RowMaj3x3Determinant(A[0], A[1], A[2], A[3], A[4], A[5], A[6], A[7], A[8]);
}

// Inverse

template <class T>
inline void RowMaj3x3Inverse(const T Det, const T A0, const T A1, const T A2, const T A3, const T A4, const T A5, const T A6, const T A7, const T A8, T* Inv)
{
	if (Det == T(0))
	{
		for (int i = 0; i < 9; i++)
			Inv[i] = T(0);
		Inv[0] = T(1);
		Inv[4] = T(1);
		Inv[8] = T(1);
		return;
	}

	Inv[0] = (A4 * A8 - A5 * A7) / Det;
	Inv[1] = (A2 * A7 - A1 * A8) / Det;
	Inv[2] = (A1 * A5 - A2 * A4) / Det;
	Inv[3] = (A5 * A6 - A3 * A8) / Det;
	Inv[4] = (A0 * A8 - A2 * A6) / Det;
	Inv[5] = (A2 * A3 - A0 * A5) / Det;
	Inv[6] = (A3 * A7 - A4 * A6) / Det;
	Inv[7] = (A1 * A6 - A0 * A7) / Det;
	Inv[8] = (A0 * A4 - A1 * A3) / Det;
}

template <class T>
inline void RowMaj3x3Inverse(const T Det, const T* A, T* Inv)
{
	RowMaj3x3Inverse(Det, A[0], A[1], A[2], A[3], A[4], A[5], A[6], A[7], A[8], Inv);
}

template <class T>
inline void RowMaj3x3Inverse(const T* A, T* Inv)
{
	RowMaj3x3Inverse(RowMaj3x3Determinant(A), A, Inv);
}

// Transpose

template <class T>
inline void RowMaj3x3Transpose(const T* A, T* Transpose)
{
	for (int32 i = 0; i < 3; i++) 
	{
		for (int32 j = 0; j < 3; j++) 
		{
			Transpose[3 * j + i] = A[3 * i + j];
		}
	}
}

template <class T,class TV>
inline TV RowMaj3x3Multiply(const T* A, const TV& x)
{
	TV Result(0,0,0);
	for (int32 i = 0; i < 3; i++) 
	{
		for (int32 j = 0; j < 3; j++) 
		{
			Result[i] += A[3 * i + j] * x[j];
		}
	}
	return Result;
}

template <class T, class TV>
inline TV RowMaj3x3RobustSolveLinearSystem(const T* A, const TV& b)
{
	T Cofactor11 = A[4] * A[8] - A[7] * A[5];
	T Cofactor12 = A[7] * A[2] - A[1] * A[8];
	T Cofactor13 = A[1] * A[5] - A[4] * A[2];
	T Determinant = A[0] * Cofactor11 + A[3] * Cofactor12 + A[6] * Cofactor13;

	T Matrix[9] =
		{ Cofactor11, Cofactor12, Cofactor13,
		  A[6] * A[5] - A[3] * A[8], A[0] * A[8] - A[6] * A[2], A[3] * A[2] - A[0] * A[5],
		  A[3] * A[7] - A[6] * A[4], A[6] * A[1] - A[0] * A[7], A[0] * A[4] - A[3] * A[1] };
	TV UnscaledResult = RowMaj3x3Multiply(&Matrix[0], b);

	T RelativeTolerance = static_cast<T>(TNumericLimits<float>::Min()) * FMath::Max3(FMath::Abs(UnscaledResult[0]), FMath::Abs(UnscaledResult[1]), FMath::Abs(UnscaledResult[2]));
	if (FMath::Abs(Determinant) <= RelativeTolerance)
	{
		RelativeTolerance = FMath::Max(RelativeTolerance, static_cast<T>(TNumericLimits<float>::Min()));
		Determinant = Determinant >= 0 ? RelativeTolerance : -RelativeTolerance;
	}
	return UnscaledResult / Determinant;
}

template <class T, class TV=FVector3f, class TV_INT=FIntVector4, int32 d=3>
void
ComputeDeInverseAndElementMeasures(
	const TArray<TV_INT>& Mesh, 
	const TArray<TV>& X, 
	TArray<T>& De_inverse, 
	TArray<T>& measure)
{
	De_inverse.SetNumUninitialized(d * d * Mesh.Num());
	measure.SetNumUninitialized(Mesh.Num());

	const int32* MeshPtr = &Mesh[0][0]; // Assumes Mesh is a continuguous array of int32.

	//PhysicsParallelForRange(Mesh.Num(), [&](const int32 eBegin, const int32 eEnd)
	const int32 eBegin = 0; const int32 eEnd = Mesh.Num();
	{
		T Inv[d*d];
		for(int32 e=eBegin; e < eEnd; e++)
		{
			const TV_INT& Elem = Mesh[e];
			T De[9] =
			{
				X[Elem[1]][0] - X[Elem[0]][0], X[Elem[2]][0] - X[Elem[0]][0], X[Elem[3]][0] - X[Elem[0]][0],
				X[Elem[1]][1] - X[Elem[0]][1], X[Elem[2]][1] - X[Elem[0]][1], X[Elem[3]][1] - X[Elem[0]][1],
				X[Elem[1]][2] - X[Elem[0]][2], X[Elem[2]][2] - X[Elem[0]][2], X[Elem[3]][2] - X[Elem[0]][2]
			};

			T Det = RowMaj3x3Determinant(De);
			measure[e] = Det / (T)6;

			RowMaj3x3Inverse(Det, De, Inv);

			for (int32 i = 0; i < d*d; i++) 
			{
				De_inverse[d*d*e+i] = Inv[i];
			}
		}
	}
	//}, 
	//100,
	//Mesh.Num() < 100);
}

template <class T>
void Fill(TArray<T>& Array, const T Value)
{ 
	for (int32 Idx = 0; Idx < Array.Num(); ++Idx) 
		Array[Idx] = Value; 
}

inline int32 MinFlatIndex(const TArray<int32>& ElemIdx, const TArray<int32>& LocalIdx)
{
	int32 MinIdx = 0;
	for (int32 i = 1; i < ElemIdx.Num(); i++)
	{
		if (ElemIdx[i] < ElemIdx[MinIdx])
			MinIdx = i;
	}
	return (ElemIdx[MinIdx] * 4) + LocalIdx[MinIdx];
}

template <class T, class TV, class TV_INT = FIntVector4, int d = 3>
void PoissonSolve(const TArray<int32>& InConstrainedNodes, 
	const TArray<T>& ConstrainedWeights, 
	const TArray<TV_INT>& Mesh,
	const TArray<TV>& X,
	const int32 MaxItCG,
	const T CGTol,
	TArray<T>& Weights)
{
	TArray<T> De_inverse;
	TArray<T> measure;
	ComputeDeInverseAndElementMeasures(Mesh, X, De_inverse, measure);

	TArray<TArray<int32>> IncidentElementsLocalIndex;
	TArray<TArray<int32>> IncidentElements = Chaos::Utilities::ComputeIncidentElements<4>(Mesh, &IncidentElementsLocalIndex);
	
	auto ProjectBCs = [&InConstrainedNodes] (TArray<T>& U)
	{
		for (int32 i = 0; i < InConstrainedNodes.Num(); i++)
		{
			U[InConstrainedNodes[i]] = (T)0.;
		}
	};

	auto MultiplyLaplacian = [&](TArray<T>& LU, const TArray<T>& U)
	{
		Laplacian(
			Mesh,
			IncidentElements,
			IncidentElementsLocalIndex,
			De_inverse,
			measure,
			U,
			LU);
		ProjectBCs(LU);
	};

	Fill(Weights, T(0));

	TArray<T> InitialGuess;
	InitialGuess.Init((T)0., Weights.Num());
	for (int32 i = 0; i < InConstrainedNodes.Num(); i++)
	{
		InitialGuess[InConstrainedNodes[i]] = ConstrainedWeights[i];
	}
	TArray<T> MinusResidual;
	MinusResidual.Init((T)0., Weights.Num());
	MultiplyLaplacian(MinusResidual, InitialGuess);

	TArray<T> MinusDw;
	MinusDw.Init((T)0., Weights.Num());
	Chaos::LanczosCG(MultiplyLaplacian, MinusDw, MinusResidual, MaxItCG, CGTol, false);

	for (int32 i = 0; i < InitialGuess.Num(); i++)
	{
		Weights[i] = InitialGuess[i] - MinusDw[i];
	}

}

template <class T, class TV_INT=FIntVector4, int d=3>
void 
Laplacian(
	const TArray<TV_INT>& Mesh,
	const TArray<TArray<int32>>& IncidentElements,
	const TArray<TArray<int32>>& IncidentElementsLocalIndex,
	const TArray<T>& De_inverse,
	const TArray<T>& measure,
	const TArray<T>& u,
	TArray<T>& Lu)
{
	TArray<Chaos::TVector<T,d>> grad_Nie_hat; grad_Nie_hat.SetNumUninitialized(d + 1);
	grad_Nie_hat[0] = { T(-1),T(-1),T(-1) };
	grad_Nie_hat[1] = { T(1),T(0),T(0) };
	grad_Nie_hat[2] = { T(0),T(1),T(0) };
	grad_Nie_hat[3] = { T(0),T(0),T(1) };

	Lu.SetNum(u.Num());
	Fill(Lu, T(0));

	//first compute element contributions
	TArray<T> element_contributions; element_contributions.SetNum(Mesh.Num() * 4);
	const int32* MeshPtr = &Mesh[0][0]; // Assumes Mesh is a contiguous array of int32.
	PhysicsParallelFor(Mesh.Num(), [&](const int32 e)
	{
		T Deinv[d * d]{
			De_inverse[d * d * e + 0],
			De_inverse[d * d * e + 1],
			De_inverse[d * d * e + 2],
			De_inverse[d * d * e + 3],
			De_inverse[d * d * e + 4],
			De_inverse[d * d * e + 5],
			De_inverse[d * d * e + 6],
			De_inverse[d * d * e + 7],
			De_inverse[d * d * e + 8],
		};
		T De_inverse_transpose[d * d];
		RowMaj3x3Transpose(Deinv, De_inverse_transpose);

		const TV_INT& Elem = Mesh[e];

		TVector<T, d> grad_Nie;
		for (int32 ie = 0; ie < d+1; ie++)
		{
			grad_Nie = RowMaj3x3Multiply(De_inverse_transpose, grad_Nie_hat[ie]);
			for (int32 je = 0; je < d+1; je++)
			{
				Chaos::TVector<T, d> grad_Nje = RowMaj3x3Multiply(De_inverse_transpose, grad_Nie_hat[je]);
				T Ae_ieje = Chaos::TVector<T, d>::DotProduct(grad_Nie, grad_Nje) * measure[e];
				element_contributions[(d + 1) * e + ie] += Ae_ieje * u[Elem[je]];
			}
		}
	});

	//now gather from elements
	PhysicsParallelFor(IncidentElements.Num(), [&](const int32 i)
	{
		if (!IncidentElements[i].Num())
		{
			return;
		}
		const TArray<int32>& IncidentElements_i = IncidentElements[i];
		const TArray<int32>& IncidentElementsLocalIndex_i = IncidentElementsLocalIndex[i];
		const int32 p = MeshPtr[MinFlatIndex(IncidentElements_i, IncidentElementsLocalIndex_i)];
		for (int32 e = 0; e < IncidentElements_i.Num(); e++)
		{
			const int32 Elem = IncidentElements_i[e];
			const int32 Local = IncidentElementsLocalIndex_i[e];
			Lu[p] += element_contributions[(Elem * 4) + Local];
		}
	});
}

template<class T, class TV_INT, int d=3>
T 
LaplacianEnergy(
	const TArray<TV_INT>& Mesh,
	const TArray<T>& De_inverse,
	const TArray<T>& measure,
	const TArray<T>& u)
{
	TArray<TVector<T, 3>> grad_Nie_hat; grad_Nie_hat.SetNumUninitialized(d + 1);
	grad_Nie_hat[0] = { T(-1),T(-1),T(-1) };
	grad_Nie_hat[1] = { T(1),T(0),T(0) };
	grad_Nie_hat[2] = { T(0),T(1),T(0) };
	grad_Nie_hat[3] = { T(0),T(0),T(1) };

	//first compute element contributions in parallel
	TArray<T> element_contributions; element_contributions.SetNum(Mesh.Num() * 4);

	//PhysicsParallelForRange(Mesh.Num(), [&](const int32 eBegin, const int32 eEnd)
	const int32 eBegin = 0;
	const int32 eEnd = Mesh.Num();
	{
		for(int32 e=eBegin; e < eEnd; e++)
		{
			T Deinv[d * d]{
				De_inverse[d * d * e + 0],
				De_inverse[d * d * e + 1],
				De_inverse[d * d * e + 2],
				De_inverse[d * d * e + 3],
				De_inverse[d * d * e + 4],
				De_inverse[d * d * e + 5],
				De_inverse[d * d * e + 6],
				De_inverse[d * d * e + 7],
				De_inverse[d * d * e + 8],
			};
			T De_inverse_transpose[d * d];
			RowMaj3x3Transpose(Deinv, De_inverse_transpose);

			const TV_INT& Elem = Mesh[e];
			TVector<T, d> grad_Nie;
			for (int32 ie = 0; ie < d + 1; ie++) 
			{
				grad_Nie = RowMaj3x3Multiply(De_inverse_transpose, grad_Nie_hat[ie]);
				for (int32 je = 0; je < d + 1; je++) 
				{
					Chaos::TVector<T, 3> grad_Nje = RowMaj3x3Multiply(De_inverse_transpose, grad_Nie_hat[je]);

					T Ae_ieje = TVector<T,d>::DotProduct(grad_Nie, grad_Nje) * measure[e];
					element_contributions[e] += u[Elem[ie]] * Ae_ieje * u[Elem[je]];
				}
			}
		}
	}/*,
	100,
	Mesh.Num() < 100);*/

	//now gather from elements
	T result = T(0);
	for (int32 i = 0; i < Mesh.Num(); ++i) 
	{
		result += element_contributions[i];
	}
	return T(.5) * result;
}




template <class T, class TV, class TV_INT, int d=3>
void 
ComputeFiberField(
	const TArray<TV_INT>& Mesh,
	const TArray<TV>& Vertices,
	const TArray<TArray<int32>>& IncidentElements,
	const TArray<TArray<int32>>& IncidentElementsLocalIndex,
	const TArray<int32>& Origins,
	const TArray<int32>& Insertions,
	TArray<TV>& Directions,
	TArray<T>& ScalarField,
	const int32 MaxIt=100,
	const T Tol=T(1e-7))
{
	// Define bc lambda
	auto set_bcs = [&Origins, &Insertions](TArray<T>& minus_u_bc, const T scale, bool zero_non_boundary = true)
	{
		if (zero_non_boundary)
			Fill(minus_u_bc, T(0));
		// origin points have bc = scale
		for (int32 i = 0; i < Origins.Num(); i++)
			minus_u_bc[Origins[i]] = scale;
		// insertion points have bc = 2*scale
		for (int32 i = 0; i < Insertions.Num(); i++)
			minus_u_bc[Insertions[i]] = 2*scale;
	};

	// Origin and insertion points are fixed
	auto proj_bcs = [&Origins, &Insertions](TArray<T>& u)
	{
		for (int32 i = 0; i < Origins.Num(); i++)
			u[Origins[i]] = T(0);
		for (int32 i = 0; i < Insertions.Num(); i++)
			u[Insertions[i]] = T(0);
	};

	TArray<T> u; u.SetNumZeroed(Vertices.Num());

	// Initialize FEM meta data
	TArray<T> De_inverse;
	TArray<T> measure;
	Chaos::ComputeDeInverseAndElementMeasures<T>(Mesh, Vertices, De_inverse, measure);

	// Set BCs and RHS
	TArray<T> negative_u_bc; negative_u_bc.SetNumZeroed(u.Num());
	TArray<T> rhs; rhs.SetNumZeroed(u.Num());
	set_bcs(negative_u_bc, T(-1), true);

	Chaos::Laplacian(Mesh, IncidentElements, IncidentElementsLocalIndex, De_inverse, measure, negative_u_bc, rhs);
	proj_bcs(rhs);

	auto mult = [&](TArray<T>& out, const TArray<T>& in)
	{
		TArray<T> proj_in = in;
		proj_bcs(proj_in);
		Laplacian(Mesh, IncidentElements, IncidentElementsLocalIndex, De_inverse, measure, proj_in, out);
		proj_bcs(out);
	};

	//solve system
	Chaos::LanczosCG(mult, u, rhs, MaxIt, Tol, true);
	set_bcs(u, T(1), false);

	//compute directions from scalar gradient
	Directions.SetNumUninitialized(Mesh.Num());
	Fill(Directions, TV(0));
	ScalarField = u;

	TArray<TV> grad_Nie_hat; grad_Nie_hat.SetNumUninitialized(d + 1);
	grad_Nie_hat[0] = { T(-1),T(-1),T(-1) };
	grad_Nie_hat[1] = { T(1),T(0),T(0) };
	grad_Nie_hat[2] = { T(0),T(1),T(0) };
	grad_Nie_hat[3] = { T(0),T(0),T(1) };

	TV pseudo_direction = Vertices[Insertions[0]] - Vertices[Origins[0]];
	pseudo_direction.Normalize();

	PhysicsParallelFor(Mesh.Num(), [&](const int32 e)
	{
		T Deinv[d * d]{
			De_inverse[d * d * e + 0],
			De_inverse[d * d * e + 1],
			De_inverse[d * d * e + 2],
			De_inverse[d * d * e + 3],
			De_inverse[d * d * e + 4],
			De_inverse[d * d * e + 5],
			De_inverse[d * d * e + 6],
			De_inverse[d * d * e + 7],
			De_inverse[d * d * e + 8],
		};
		T De_inverse_transpose[d * d];
		RowMaj3x3Transpose(Deinv, De_inverse_transpose);

		const TV_INT& Elem = Mesh[e];
		TV gradient(0);
		TVector<T, d> grad_Nie;
		for (size_t ie = 0; ie < d + 1; ie++) 
		{
			grad_Nie = RowMaj3x3Multiply(De_inverse_transpose, grad_Nie_hat[ie]);
			gradient += grad_Nie * u[Elem[ie]];
		}
		const T Len = gradient.Length();
		if (Len > T(1e-10)) 
		{
			Directions[e] = gradient * T(1) / Len;
		}
		else if (u[Mesh[e][0]] > 0)
		{
			Directions[e] = pseudo_direction;
		}
		else
		{
			Directions[e] = gradient;
		}
	});
}

// 9 point laplacian with dirichlet boundaries
template<class TV, class T, bool NodalValues = false>
void Laplacian(const TUniformGrid<T, 3>& UniformGrid,
	const TArray<TV>& U,
	TArray<TV>& Lu)
{
	const TVec3<int32> Counts = NodalValues ? UniformGrid.NodeCounts() : UniformGrid.Counts();

	const int32 XStride = Counts[1] * Counts[2];
	const int32 YStride = Counts[2];
	constexpr int32 ZStride = 1;
	const Chaos::TVector<TV, 3> OneOverDxSq = Chaos::TVector<TV, 3>(1) / (UniformGrid.Dx() * UniformGrid.Dx());

	Fill(Lu, TV(0));

	for (int32 I = 1; I < Counts.X - 1; ++I)
	{
		for (int32 J = 1; J < Counts.Y - 1; ++J)
		{
			for (int32 K = 1; K < Counts.Z - 1; ++K)
			{
				const int32 FlatIndex = I * XStride + J * YStride + K * ZStride;
				Lu[FlatIndex] +=
					(U[FlatIndex + XStride] - (T)2. * U[FlatIndex] + U[FlatIndex - XStride]) * OneOverDxSq[0] +
					(U[FlatIndex + YStride] - (T)2. * U[FlatIndex] + U[FlatIndex - YStride]) * OneOverDxSq[1] +
					(U[FlatIndex + ZStride] - (T)2. * U[FlatIndex] + U[FlatIndex - ZStride]) * OneOverDxSq[2];
			}
		}
	}
}

// Solve Poisson on a Uniform Grid (with standard 9-point Laplacian).
// Weights is a flattened array (using same indexing as UniformGrid and ArrayND) and the output solution.
// InConstrainedNodes is a list of flattened index nodes that are boundary conditions. ConstrainedWeights is a parallel array of the constrained values.
template<class TV, class T, bool NodalValues = false>
void PoissonSolve(const TArray<int32>& InConstrainedNodes,
	const TArray<TV>& ConstrainedWeights,
	const TUniformGrid<T, 3>& UniformGrid,
	const int32 MaxItCG,
	const TV CGTol,
	TArray<TV>& Weights,
	bool bCheckResidual = false,
	int32 MinParallelBatchSize = 1000)
{
	auto ProjectBCs = [&InConstrainedNodes](TArray<TV>& U)
	{
		for (int32 i = 0; i < InConstrainedNodes.Num(); i++)
		{
			U[InConstrainedNodes[i]] = (T)0.;
		}
	};

	auto MultiplyLaplacian = [&ProjectBCs, &UniformGrid](TArray<TV>& LU, const TArray<TV>& U)
	{
		Laplacian<TV, T, NodalValues>(
			UniformGrid,
			U,
			LU);
		ProjectBCs(LU);
	};

	Fill(Weights, TV(0));

	TArray<TV> InitialGuess;
	InitialGuess.Init((TV)0., Weights.Num());
	for (int32 i = 0; i < InConstrainedNodes.Num(); i++)
	{
		InitialGuess[InConstrainedNodes[i]] = ConstrainedWeights[i];
	}
	TArray<TV> MinusResidual;
	MinusResidual.Init((TV)0., Weights.Num());
	MultiplyLaplacian(MinusResidual, InitialGuess);

	TArray<TV> MinusDw;
	MinusDw.Init((TV)0., Weights.Num());
	Chaos::LanczosCG(MultiplyLaplacian, MinusDw, MinusResidual, MaxItCG, CGTol, bCheckResidual, MinParallelBatchSize);
	
	for (int32 i = 0; i < InitialGuess.Num(); i++)
	{
		Weights[i] = InitialGuess[i] - MinusDw[i];
	}
}


} // namespace Chaos
