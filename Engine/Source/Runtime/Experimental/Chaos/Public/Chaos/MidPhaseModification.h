// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/Collision/ParticlePairMidPhase.h"
#include "ParticleHandleFwd.h"

namespace Chaos
{
	namespace Private
	{
		class FCollisionConstraintAllocator;	// Container for solver's constraints and midphase pairs
	}
	class FParticlePairMidPhase;			// The underlying midphase type
	class FMidPhaseModifier;			// Modifier for a particular midphase
	class FMidPhaseModifierAccessor;		// Access class for all midphase pair modifiers

	/*
	 * Wrapper around a MidPhase object which exposes limited access and manipulation functions.
	 * Also contains the Modifier, which is responsible for iterating over PairModifiers, and
	 * storing and executing requested modifications.
	 */
	class FMidPhaseModifier
	{
	public:
		FMidPhaseModifier()
			: MidPhase(nullptr)
			, Accessor(nullptr)
		{}

		FMidPhaseModifier(FParticlePairMidPhase* InMidPhase, FMidPhaseModifierAccessor* InAccessor)
			: MidPhase(InMidPhase)
			, Accessor(InAccessor)
		{}

		// Since a modifier can be invalid (ie, null midphase and/or accessor), make it
		// castable to bool so that users can check validity.
		bool IsValid() const
		{
			return
				MidPhase != nullptr &&
				Accessor != nullptr;
		}

		//
		// Modifying functions
		//

		// Disable this midphase entirely
		CHAOS_API void Disable();

		// Disable CCD for this pair
		CHAOS_API void DisableCCD();

		// Disable convex optimization for this pair
		CHAOS_API void DisableConvexOptimization();

		//
		// Accessor functions
		//

		CHAOS_API void GetParticles(FGeometryParticleHandle** Particle0, FGeometryParticleHandle** Particle1) const;

		CHAOS_API FGeometryParticleHandle* GetOtherParticle(const FGeometryParticleHandle* InParticle);

		CHAOS_API const FGeometryParticleHandle* GetOtherParticle(const FGeometryParticleHandle* InParticle) const;

	private:

		FParticlePairMidPhase* MidPhase;
		FMidPhaseModifierAccessor* Accessor;
	};

	/*
	 * Class for iterating over midphases involving a specific particle
	 */
	class FMidPhaseModifierParticleIterator
	{
	public:
		FMidPhaseModifierParticleIterator(
			FMidPhaseModifierAccessor* InAccessor,
			TGeometryParticleHandle<FReal, 3>* InParticle,
			int32 InMidPhaseIndex = 0)
			: Particle(InParticle)
			, Accessor(InAccessor)
			, MidPhaseIndex(InMidPhaseIndex)
			, PairModifier(Particle->ParticleCollisions().GetMidPhase(MidPhaseIndex), Accessor)
		{ }

		FMidPhaseModifier& operator*()
		{
			return PairModifier;
		}

		FMidPhaseModifier* operator->()
		{
			return &PairModifier;
		}

		FMidPhaseModifierParticleIterator& operator++()
		{
			++MidPhaseIndex;
			PairModifier = FMidPhaseModifier(Particle->ParticleCollisions().GetMidPhase(MidPhaseIndex), Accessor);
			return *this;
		}

		explicit operator bool() const
		{
			return MidPhaseIndex < Particle->ParticleCollisions().Num();
		}

		bool operator==(const FMidPhaseModifierParticleIterator& Other) const
		{
			return MidPhaseIndex == Other.MidPhaseIndex;
		}

		bool operator!=(const FMidPhaseModifierParticleIterator& Other) const
		{
			return !operator==(Other);
		}

	private:
		TGeometryParticleHandle<FReal, 3>* Particle;
		FMidPhaseModifierAccessor* Accessor;
		int32 MidPhaseIndex;
		FMidPhaseModifier PairModifier;

		friend class FMidPhaseModifierParticleRange;
	};

	/*
	* A range-iterable object for accessing all mid-phases which include a particular particle
	*/
	class FMidPhaseModifierParticleRange
	{
	public:
		CHAOS_API FMidPhaseModifierParticleIterator begin() const;
		CHAOS_API FMidPhaseModifierParticleIterator end() const;

	private:
		FMidPhaseModifierParticleRange(
			FMidPhaseModifierAccessor* InAccessor, FGeometryParticleHandle* InParticle)
			: Accessor(InAccessor)
			, Particle(InParticle) { }

		FMidPhaseModifierAccessor* Accessor;
		FGeometryParticleHandle* Particle;

		// Befriend the accessor so that it alone can instantiate this range object
		friend class FMidPhaseModifierAccessor;
	};

	/*
	 * Provides interface for accessing midphase pair modifiers
	 */
	class FMidPhaseModifierAccessor
	{
	public:
		FMidPhaseModifierAccessor(FPBDRigidsEvolutionGBF& InEvolution)
			: Evolution(InEvolution)
		{ }

		// Get the total number of mid phases for all particles
		CHAOS_API int32 GetNumMidPhases() const;

		// Get the number of mid phases involving a particular particle
		CHAOS_API int32 GetNumMidPhases(const FGeometryParticleHandle* Particle) const;

		// Get an object which allows for range iteration over the list of
		// midphases for a particle
		CHAOS_API FMidPhaseModifierParticleRange GetMidPhases(FGeometryParticleHandle* Particle);

		// Get a midphase modifier for a particular object pair
		CHAOS_API FMidPhaseModifier GetMidPhase(FGeometryParticleHandle* Particle0, FGeometryParticleHandle* Particle1);

		// Provide visitor access to all midphase modifiers
		CHAOS_API void VisitMidPhases(const TFunction<void(FMidPhaseModifier&)>& Visitor);

	private:
		friend class FMidPhaseModifier;

		FPBDRigidsEvolutionGBF& GetEvolution() { return Evolution; }

		FPBDRigidsEvolutionGBF& Evolution;
	};
}
