// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once 
#include "Chaos/Matrix.h"

namespace Chaos::Softs
{
template <typename T>
T PsiNeohookeanMM(const Chaos::PMatrix<T, 3, 3>& F, const T mu, const T lambda);

template <typename T>
void PNeohookeanMM(const Chaos::PMatrix<T, 3, 3>& Fe, const T mu, const T lambda, Chaos::PMatrix<T, 3, 3>& P);

}

