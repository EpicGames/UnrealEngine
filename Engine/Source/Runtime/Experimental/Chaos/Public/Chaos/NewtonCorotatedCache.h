// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once 
#include "Chaos/Matrix.h"

namespace Chaos::Softs
{

template <typename T>
struct CorotatedCache 
{
	TArray<T> JFInvTCache, RCache;
	TArray<T> DInvCache;

	T JCache;


	CorotatedCache()
	{
		JFInvTCache.Init((T)0., 3*3);
		RCache.Init((T)0., 3*3);
		DInvCache.Init((T)0., 3*3);
	};

	void UpdateCache(const Chaos::PMatrix<T, 3, 3>& F);
	void P(const Chaos::PMatrix<T, 3, 3>& F, const T mu, const T lambda, Chaos::PMatrix<T, 3, 3>& P);
	void deltaP(const Chaos::PMatrix<T, 3, 3>& F, const Chaos::PMatrix<T, 3, 3>& dF, const T mu, const T lambda, Chaos::PMatrix<T, 3, 3>& dP);

};

template <typename T>
T PsiCorotated(const Chaos::PMatrix<T, 3, 3>& F, const T mu, const T lambda);

template <typename T>
void PCorotated(const Chaos::PMatrix<T, 3, 3>& F, const T mu, const T lambda, Chaos::PMatrix<T, 3, 3>& P);

}
