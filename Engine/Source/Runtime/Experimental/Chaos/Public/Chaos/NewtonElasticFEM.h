// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/ArrayCollection.h"
#include "Chaos/PBDActiveView.h"
#include "Chaos/PBDSoftsEvolutionFwd.h"
#include "Chaos/PBDSoftsSolverParticles.h"
#include "Chaos/KinematicGeometryParticles.h"
#include "Chaos/VelocityField.h"

namespace Chaos::Softs
{


template <typename T, typename ParticleType>
class ElasticFEM 
{
public:
	const TArray<TVector<int32, 4>>& Mesh;
	TArray<T> DmInverse;
	TArray<T> Measure;
	TArray<TVector<T, 3>> ElementForces;  // extra storage in case of parallel force and force differential computation
	int32 Np;// the total number of particles in the arrays that mesh is indexing
	TArray<T> MNodalMass;
	TArray<TArray<TVector<int32, 2>>> MIncidentElements;

	ElasticFEM(
		const ParticleType& InParticles,
		const TArray<TVector<int32, 4>>& InMesh,
		const TArray<TArray<TVector<int32, 2>>>& IncidentElementsIn
	)
		: Mesh(InMesh), MIncidentElements(IncidentElementsIn)
	{
		
		Np = InParticles.Size();
		DmInverse.Init((T)0., 9 * Mesh.Num());
		Measure.Init((T)0., Mesh.Num());
		for (int e = 0; e < Mesh.Num(); e++)
		{
			PMatrix<T, 3, 3> Dm = DsInit(e, InParticles);
			PMatrix<T, 3, 3> DmInv = Dm.Inverse();
			for (int r = 0; r < 3; r++) {
				for (int c = 0; c < 3; c++) {
					DmInverse[(3 * 3) * e + 3 * r + c] = DmInv.GetAt(r, c);
				}
			}

			Measure[e] = Dm.Determinant() / (T)6.;

			//part of preprocessing: if inverted element is found, 
			//invert it so that the measure is positive
			if (Measure[e] < (T)0.)
			{

				Measure[e] = -Measure[e];
			}
		}
	}

	void ComputeNodalMass(const T Density)
	{
		MNodalMass.Init((T)0., Np);
		for (int32 e = 0; e < Mesh.Num(); e++) 
		{
			T NodeMassContribution = Density * Measure[e] / (T)4.;
			for (int32 i = 0; i < 4; i++) 
			{
				MNodalMass[Mesh[e][i]] += NodeMassContribution;
			}
		}
	}

	PMatrix<T, 3, 3> DsInit(const int e, const ParticleType& InParticles) const 
	{
		PMatrix<T, 3, 3> Result((T)0.);
		for (int i = 0; i < 3; i++) {
			for (int c = 0; c < 3; c++) {
				Result.SetAt(c, i, InParticles.GetX(Mesh[e][i + 1])[c] - InParticles.GetX(Mesh[e][0])[c]);
			}
		}
		return Result;
	}


	PMatrix<T, 3, 3> Ds(const int e, const ParticleType& InParticles) const 
	{
		PMatrix<T, 3, 3> Result((T)0.);
		for (int i = 0; i < 3; i++) {
			for (int c = 0; c < 3; c++) {
				Result.SetAt(c, i, InParticles.P(Mesh[e][i + 1])[c] - InParticles.P(Mesh[e][0])[c]);
			}
		}
		return Result;
	}

	PMatrix<T, 3, 3> Ds(const int e, const TArray<TVector<T, 3>>& InParticles) const 
	{
		PMatrix<T, 3, 3> Result((T)0.);
		for (int i = 0; i < 3; i++) {
			for (int c = 0; c < 3; c++) {
				Result.SetAt(c, i, InParticles[Mesh[e][i + 1]][c] - InParticles[Mesh[e][0]][c]);
			}
		}
		return Result;
	}


	PMatrix<T, 3, 3> F(const int e, const ParticleType& InParticles) const {
		return ElementDmInv(e) * Ds(e, InParticles);
	}

	PMatrix<T, 3, 3> F(const int e, const TArray<TVector<T, 3>>& InParticles) const {
		return ElementDmInv(e) * Ds(e, InParticles);
	}

	PMatrix<T, 3, 3> ElementDmInv(const int e) const 
	{
		PMatrix<T, 3, 3> DmInv((T)0.);
		for (int r = 0; r < 3; r++) {
			for (int c = 0; c < 3; c++) {
				DmInv.SetAt(r, c, DmInverse[(3 * 3) * e + 3 * r + c]);
			}
		}
		return DmInv;
	}

	template <typename Func>
	void AddInternalElasticForce(const ParticleType& InParticles, Func P, TArray<TVector<T, 3>>& Force, const TArray<TArray<TVector<int32, 2>>>* IncidentElements = nullptr, T Scale = (T)1., const TArray<T>* NodalMass = 0)
	{
		//compute the force per node from the mesh, DmInv and P
		//incident elements optionally used for parallelism
		//scale and nodal mass can be used to multiply/divide the force entries so that this can be used with time stepping

		Force.Init(TVector<T, 3>((T)0.), Force.Num());

		if (NodalMass)
			ensureMsgf(NodalMass->Num() == InParticles.Size(), TEXT("NewtonEvolution::AddInternalElasticForce: mass and position sized inconistently."));

		if (!IncidentElements) {
			for (int32 e = 0; e < Mesh.Num() ; e++) {
				PMatrix<T, 3, 3> Fe = F(e, InParticles);
				PMatrix<T, 3, 3> Pe(0.f);
				P(Fe, Pe, e);
				PMatrix<T, 3, 3> g = -Measure[e] * (ElementDmInv(e).GetTransposed()) * Pe;
				for (int32 ie = 0; ie < 3; ie++) {
					T InverseMass = 1.f;
					if (NodalMass)
						InverseMass = 1.f / NodalMass -> operator[](Mesh[e][ie+1]);
					for (int32 c = 0; c < 3; c++) {
						Force[Mesh[e][ie+1]][c] += Scale * InverseMass * g.GetAt(c, ie);
					}
				}
				T InverseMass = 1.f;
				if (NodalMass)
					InverseMass = 1.f / NodalMass->operator[](Mesh[e][0]);
				for (int32 c = 0; c < 3; c++) {
					for (int32 h = 0; h < 3; h++) {
						Force[Mesh[e][0]][c] -= Scale * InverseMass * g.GetAt(c, h);
					}
				}
			}
		}
		else {
			ElementForces.Init(TVector<T, 3>(0.f), 4 * Mesh.Num());
			PhysicsParallelFor(Mesh.Num(), [&](const int32 e)
				{
					PMatrix<T, 3, 3> Fe = F(e, InParticles);
					PMatrix<T, 3, 3> Pe(0.f);
					P(Fe, Pe, e);
					PMatrix<T, 3, 3> g = -Measure[e] * (ElementDmInv(e).GetTransposed()) * Pe;
					for (int32 ie = 0; ie < 3; ie++) {
						for (int32 c = 0; c < 3; c++) {
							ElementForces[4 * e + ie + 1][c] += g.GetAt(c, ie);
						}
					}
					for (int32 c = 0; c < 3; c++) {
						for (int32 h = 0; h < 3; h++) {
							ElementForces[4 * e][c] -= g.GetAt(c, h);
						}
					}
				});


			if (!NodalMass) {
				//#pragma omp parallel for
				PhysicsParallelFor(IncidentElements->Num(), [&](const int32 i)
					{
						for (int32 e = 0; e < (*IncidentElements)[i].Num(); e++) {
							for (int32 c = 0; c < 3; c++) {
								Force[Mesh[(*IncidentElements)[i][e][0]][(*IncidentElements)[i][e][1]]][c] += Scale * ElementForces[4* (*IncidentElements)[i][e][0]+ (*IncidentElements)[i][e][1]][c];
							}
						}
					});
			}
			else {
				//#pragma omp parallel for
				PhysicsParallelFor(IncidentElements->Num(), [&](const int32 i)
					{
						for (int32 e = 0; e < (*IncidentElements)[i].Num(); e++) {
							for (int32 c = 0; c < 3; c++) {
								Force[Mesh[(*IncidentElements)[i][e][0]][(*IncidentElements)[i][e][1]]][c] += Scale * ElementForces[4 * (*IncidentElements)[i][e][0] + (*IncidentElements)[i][e][1]][c] / (*NodalMass)[Mesh[(*IncidentElements)[i][e][0]][(*IncidentElements)[i][e][1]]];
							}
						}
					});
			}
		}
	}

	template <typename Func>
	void AddNegativeInternalElasticForceDifferential(const ParticleType& InParticles, Func dP, const TArray<TVector<T, 3>>& DeltaParticles, TArray<TVector<T, 3>>& ndf, const TArray<TArray<TVector<int32, 2>>>* IncidentElements = nullptr)
	{
		//matrix free elastic potential energy Hessian based differentials
		//compute the force differential per node from the mesh, DmInv and dP
		//incident elements optionally used for parallelism

		ndf.Init(TVector<T, 3>(0.f), ndf.Num());

		if (!IncidentElements) {
			for (int32 e = 0; e < Mesh.Num() ; e++) {
				PMatrix<T, 3, 3> Fe = F(e, InParticles), dFe = F(e, DeltaParticles);
				PMatrix<T, 3, 3> dPe;
				dP(Fe, dFe, dPe, e);
				PMatrix<T, 3, 3> dg = Measure[e] * (ElementDmInv(e).GetTransposed()) * dPe;
				for (int32 ie = 0; ie < 3; ie++) {
					for (int32 c = 0; c < 3; c++) {
						ndf[Mesh[e][ie+1]][c] += dg.GetAt(c, ie);
					}
				}
				for (int32 c = 0; c < 3; c++) {
					for (int32 h = 0; h < 3; h++) {
						ndf[Mesh[e][0]][c] -= dg.GetAt(c, h);
					}
				}
			}
		}
		else {
			ElementForces.Init(TVector<T, 3>(0.f), Mesh.Num() * 4);
			PhysicsParallelFor(Mesh.Num(), [&](const int32 e) {
				//for (int32 e = 0; e < int32(Mesh.Num() ); ++e) {
				PMatrix<T, 3, 3> Fe = F(e, InParticles), dFe = F(e, DeltaParticles);
				PMatrix<T, 3, 3> dPe;
				dP(Fe, dFe, dPe, e);
				PMatrix<T, 3, 3> dg = Measure[e] * (ElementDmInv(e).GetTransposed()) * dPe;
				for (int32 ie = 0; ie < 3; ie++)
				{
					for (int32 c = 0; c < 3; c++)
					{
						ElementForces[4 * e + ie + 1][c] += dg.GetAt(c, ie);
					}
				}
				for (int32 c = 0; c < 3; c++) {
					for (int32 h = 0; h < 3; h++) {
						ElementForces[4 * e][c] -= dg.GetAt(c, h);
					}
				}
				});

			//#pragma omp parallel for
			for (int32 i = 0; i < int32(IncidentElements->Num()); ++i) {
				for (int32 e = 0; e < (*IncidentElements)[i].Num(); e++) {
					for (int32 c = 0; c < 3; c++) {
						ndf[Mesh[(*IncidentElements)[i][e][0]][(*IncidentElements)[i][e][1]]][c] += ElementForces[4 * (*IncidentElements)[i][e][0] + (*IncidentElements)[i][e][1]][c];
					}
				}
			}
		}
	}



};

}
