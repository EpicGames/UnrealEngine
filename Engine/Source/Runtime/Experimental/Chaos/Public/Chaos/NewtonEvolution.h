// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/ArrayCollection.h"
#include "Chaos/PBDActiveView.h"
#include "Chaos/PBDSoftsEvolutionFwd.h"
#include "Chaos/PBDSoftsSolverParticles.h"
#include "Chaos/NewtonElasticFEM.h"
#include "Chaos/NewtonCorotatedCache.h"
#include "Chaos/SoftsSolverCollisionParticles.h"
#include "Chaos/VelocityField.h"

namespace Chaos::Softs
{

struct ConsModelCaches {
	TArray<Chaos::Softs::CorotatedCache<FSolverReal>> CorotatedCache;
	//TArray<CorotatedFiberCache> CorotatedFiberCache;
};

class FNewtonEvolution : public TArrayCollection
{
public:
	// TODO: Tidy up this constructor (and update Headless Chaos)
	CHAOS_API FNewtonEvolution(
		FSolverParticles&& InParticles,
		FSolverCollisionParticles&& InGeometryParticles,
		TArray<TVec3<int32>>&& CollisionTriangles,
		const TArray<TVector<int32, 4>>& InMesh,
		TArray<TArray<TVector<int32, 2>>>&& InIncidentElements,
		int32 NumNewtonIterations = 5,
		int32 NumCGIterations = 20,
		const TArray<int32>& ConstrainedVertices = TArray<int32>(),
		const TArray<FSolverVec3>& BCPositions = TArray<FSolverVec3>(),
		FSolverReal CollisionThickness = (FSolverReal)0.,
		FSolverReal SelfCollisionsThickness = (FSolverReal)0.,
		FSolverReal CoefficientOfFriction = (FSolverReal)0.,
		FSolverReal Damping = (FSolverReal)0.04,
		FSolverReal LocalDamping = (FSolverReal)0.,
		FSolverReal EMesh = (FSolverReal)1000.,
		FSolverReal NuMesh = (FSolverReal).3,
		FSolverReal NewtonTol = (FSolverReal) 1e-6,
		FSolverReal CGTolIn = (FSolverReal) 1e-8,
		bool bWriteDebugInfoIn = true);
	~FNewtonEvolution() = default;

	// Advance one time step. Filter the input time step if specified.
	CHAOS_API void AdvanceOneTimeStep(const FSolverReal Dt, const bool bSmoothDt = true);

	// Remove all particles, will also reset all rules
	CHAOS_API void ResetParticles();

	// Add particles and initialize group ids. Return the index of the first added particle.
	CHAOS_API int32 AddParticleRange(int32 NumParticles, uint32 GroupId, bool bActivate);

	// Return the number of particles of the block starting at Offset
	int32 GetParticleRangeSize(int32 Offset) const { return MParticlesActiveView.GetRangeSize(Offset); }

	// Set a block of particles active or inactive, using the index of the first added particle to identify the block.
	void ActivateParticleRange(int32 Offset, bool bActivate) { MParticlesActiveView.ActivateRange(Offset, bActivate); }

	// Particles accessors
	const FSolverParticles& Particles() const { return MParticles; }
	FSolverParticles& Particles() { return MParticles; }
	const TPBDActiveView<FSolverParticles>& ParticlesActiveView() { return MParticlesActiveView; }

	TArray<TArray<TVector<int32, 2>>>& IncidentElements() { return MIncidentElements; }

	const TArray<uint32>& ParticleGroupIds() const { return MParticleGroupIds; }

	// Remove all collision particles
	CHAOS_API void ResetCollisionParticles(int32 NumParticles = 0);

	// Add collision particles and initialize group ids. Return the index of the first added particle.
	// Use INDEX_NONE as GroupId for collision particles that affect all particle groups.
	CHAOS_API int32 AddCollisionParticleRange(int32 NumParticles, uint32 GroupId, bool bActivate);

	// Set a block of collision particles active or inactive, using the index of the first added particle to identify the block.
	void ActivateCollisionParticleRange(int32 Offset, bool bActivate) { MCollisionParticlesActiveView.ActivateRange(Offset, bActivate); }

	// Return the number of particles of the block starting at Offset
	int32 GetCollisionParticleRangeSize(int32 Offset) const { return MCollisionParticlesActiveView.GetRangeSize(Offset); }

	//// collision particles accessors
	const FSolverCollisionParticles& CollisionParticles() const { return MCollisionParticles; }
	FSolverCollisionParticles& CollisionParticles() { return MCollisionParticles; }
	const TArray<uint32>& CollisionParticleGroupIds() const { return MCollisionParticleGroupIds; }
	const TPBDActiveView<FSolverCollisionParticles>& CollisionParticlesActiveView() { return MCollisionParticlesActiveView; }


	// Reset all constraint init and rule functions.
	void ResetConstraintRules()
	{
		MConstraintInits.Reset();
		MConstraintRules.Reset();
		MPostCollisionConstraintRules.Reset();
		MConstraintInitsActiveView.Reset();
		MConstraintRulesActiveView.Reset();
		MPostCollisionConstraintRulesActiveView.Reset();
	}

	// Add constraints. Return the index of the first added constraint.
	CHAOS_API int32 AddConstraintInitRange(int32 NumConstraints, bool bActivate);
	CHAOS_API int32 AddConstraintRuleRange(int32 NumConstraints, bool bActivate);
	CHAOS_API int32 AddPostCollisionConstraintRuleRange(int32 NumConstraints, bool bActivate);

	// Return the number of particles of the block starting at Offset
	int32 GetConstraintInitRangeSize(int32 Offset) const { return MConstraintInitsActiveView.GetRangeSize(Offset); }
	int32 GetConstraintRuleRangeSize(int32 Offset) const { return MConstraintRulesActiveView.GetRangeSize(Offset); }
	int32 GetPostCollisionConstraintRuleRangeSize(int32 Offset) const { return MPostCollisionConstraintRulesActiveView.GetRangeSize(Offset); }

	// Set a block of constraints active or inactive, using the index of the first added particle to identify the block.
	void ActivateConstraintInitRange(int32 Offset, bool bActivate) { MConstraintInitsActiveView.ActivateRange(Offset, bActivate); }
	void ActivateConstraintRuleRange(int32 Offset, bool bActivate) { MConstraintRulesActiveView.ActivateRange(Offset, bActivate); }
	void ActivatePostCollisionConstraintRuleRange(int32 Offset, bool bActivate) { MPostCollisionConstraintRulesActiveView.ActivateRange(Offset, bActivate); }

	// Constraint accessors
	const TArray<TFunction<void(FSolverParticles&, const FSolverReal)>>& ConstraintInits() const { return MConstraintInits; }
	TArray<TFunction<void(FSolverParticles&, const FSolverReal)>>& ConstraintInits() { return MConstraintInits; }
	const TArray<TFunction<void(FSolverParticles&, const FSolverReal)>>& ConstraintRules() const { return MConstraintRules; }
	TArray<TFunction<void(FSolverParticles&, const FSolverReal)>>& ConstraintRules() { return MConstraintRules; }
	const TArray<TFunction<void(FSolverParticles&, const FSolverReal)>>& PostCollisionConstraintRules() const { return MPostCollisionConstraintRules; }
	TArray<TFunction<void(FSolverParticles&, const FSolverReal)>>& PostCollisionConstraintRules() { return MPostCollisionConstraintRules; }

	void SetKinematicUpdateFunction(TFunction<void(FSolverParticles&, const FSolverReal, const FSolverReal, const int32)> KinematicUpdate) { MKinematicUpdate = KinematicUpdate; }
	void SetCollisionKinematicUpdateFunction(TFunction<void(FSolverCollisionParticles&, const FSolverReal, const FSolverReal, const int32)> KinematicUpdate) { MCollisionKinematicUpdate = KinematicUpdate; }

	TFunction<void(FSolverParticles&, const FSolverReal, const int32)>& GetForceFunction(const uint32 GroupId = 0) { return MGroupForceRules[GroupId]; }
	const TFunction<void(FSolverParticles&, const FSolverReal, const int32)>& GetForceFunction(const uint32 GroupId = 0) const { return MGroupForceRules[GroupId]; }

	const FSolverVec3& GetGravity(const uint32 GroupId = 0) const { check(GroupId < TArrayCollection::Size()); return MGroupGravityAccelerations[GroupId]; }
	void SetGravity(const FSolverVec3& Acceleration, const uint32 GroupId = 0) { check(GroupId < TArrayCollection::Size()); MGroupGravityAccelerations[GroupId] = Acceleration; }

	FVelocityAndPressureField& GetVelocityAndPressureField(const uint32 GroupId = 0) { check(GroupId < TArrayCollection::Size()); return MGroupVelocityAndPressureFields[GroupId]; }
	const FVelocityAndPressureField& GetVelocityAndPressureField(const uint32 GroupId = 0) const { check(GroupId < TArrayCollection::Size()); return MGroupVelocityAndPressureFields[GroupId]; }

	int32 GetNewtonIterations() const { return MNumNewtonIterations; }
	void SetNewtonIterations(const int32 Iterations) { MNumNewtonIterations = Iterations; }

	FSolverReal GetCollisionThickness(const uint32 GroupId = 0) const { check(GroupId < TArrayCollection::Size()); return MGroupCollisionThicknesses[GroupId]; }
	void SetCollisionThickness(const FSolverReal CollisionThickness, const uint32 GroupId = 0) { check(GroupId < TArrayCollection::Size()); MGroupCollisionThicknesses[GroupId] = CollisionThickness; }

	FSolverReal GetCoefficientOfFriction(const uint32 GroupId = 0) const { check(GroupId < TArrayCollection::Size()); return MGroupCoefficientOfFrictions[GroupId]; }
	void SetCoefficientOfFriction(const FSolverReal CoefficientOfFriction, const uint32 GroupId = 0) { check(GroupId < TArrayCollection::Size()); MGroupCoefficientOfFrictions[GroupId] = CoefficientOfFriction; }

	FSolverReal GetDamping(const uint32 GroupId = 0) const { check(GroupId < TArrayCollection::Size()); return MGroupDampings[GroupId]; }
	void SetDamping(const FSolverReal Damping, const uint32 GroupId = 0) { check(GroupId < TArrayCollection::Size()); MGroupDampings[GroupId] = Damping; }

	FSolverReal GetLocalDamping(const uint32 GroupId = 0) const { check(GroupId < TArrayCollection::Size()); return MGroupLocalDampings[GroupId]; }
	void SetLocalDamping(const FSolverReal LocalDamping, const uint32 GroupId = 0) { check(GroupId < TArrayCollection::Size()); MGroupLocalDampings[GroupId] = LocalDamping; }

	bool GetUseCCD(const uint32 GroupId = 0) const { check(GroupId < TArrayCollection::Size()); return MGroupUseCCDs[GroupId]; }
	void SetUseCCD(const bool bUseCCD, const uint32 GroupId = 0) { check(GroupId < TArrayCollection::Size()); MGroupUseCCDs[GroupId] = bUseCCD; }

	UE_DEPRECATED(4.27, "Use GetCollisionStatus() instead")
		const bool Collided(int32 index) { return MCollided[index]; }

	const TArray<bool>& GetCollisionStatus() { return MCollided; }
	const TArray<FSolverVec3>& GetCollisionContacts() const { return MCollisionContacts; }
	const TArray<FSolverVec3>& GetCollisionNormals() const { return MCollisionNormals; }

	FSolverReal GetTime() const { return MTime; }
		
	template <typename Func1, typename Func2>
	void ComputeNegativeBackwardEulerResidual(const FSolverParticles& InParticles, const TArray<TArray<TVector<int32, 2>>>& IncidentElements, const TArray<FSolverReal>& NodalMass, const TArray<FSolverVec3>& Vn, const FSolverParticles& Xn, Func1 P, Func2 AddExternalForce, const FSolverReal Time, const FSolverReal Dt, TArray<FSolverVec3>& Residual);
		
	template<typename Func1, typename Func2, typename Func3, typename Func4, typename Func5, typename Func6>
	void DoNewtonStep(const int32 max_it_newton, const FSolverReal newton_tol, const int32 max_it_cg, const FSolverReal cg_tol, Func1 P, Func2 dP, const FSolverReal time, const FSolverReal dt, const TArray<TArray<TVector<int32, 2>>>& incident_elements, const TArray<FSolverReal>& nodal_mass, Func3 set_bcs, Func4 project_bcs, Func5 add_external_force, Func6 update_position_based_state, FSolverParticles& InParticles, TArray<FSolverReal>& residual_norm, bool use_cg = false, FSolverReal cg_prctg_reduce = 0, bool no_verbose = false);

	CHAOS_API void InitFEM();

	CHAOS_API void WriteOutputLog(const int32 Frame);

private:
	// Add simulation groups and set default values
	CHAOS_API void AddGroups(int32 NumGroups);
	// Reset simulation groups
	CHAOS_API void ResetGroups();
	// Selected versions of the pre-iteration updates (euler step, force, velocity field. damping updates)..
	template<bool bForceRule, bool bVelocityField, bool bDampVelocityRule>
	void PreIterationUpdate(const FSolverReal Dt, const int32 Offset, const int32 Range, const int32 MinParallelBatchSize);
	

private:
	FSolverParticles MParticles;
	TPBDActiveView<FSolverParticles> MParticlesActiveView;
	FSolverCollisionParticles MCollisionParticles;
	TPBDActiveView<FSolverCollisionParticles> MCollisionParticlesActiveView;

	TArrayCollectionArray<FSolverRigidTransform3> MCollisionTransforms;  // Used for CCD to store the initial state before the kinematic update
	TArrayCollectionArray<bool> MCollided;
	TArrayCollectionArray<uint32> MCollisionParticleGroupIds;  // Used for per group parameters for collision particles
	TArrayCollectionArray<uint32> MParticleGroupIds;  // Used for per group parameters for particles
	TArray<FSolverVec3> MCollisionContacts;
	TArray<FSolverVec3> MCollisionNormals;

	TArrayCollectionArray<FSolverVec3> MGroupGravityAccelerations;
	TArrayCollectionArray<FVelocityAndPressureField> MGroupVelocityAndPressureFields;
	TArrayCollectionArray<TFunction<void(FSolverParticles&, const FSolverReal, const int32)>> MGroupForceRules;
	TArrayCollectionArray<FSolverReal> MGroupCollisionThicknesses;
	TArrayCollectionArray<FSolverReal> MGroupCoefficientOfFrictions;
	TArrayCollectionArray<FSolverReal> MGroupDampings;
	TArrayCollectionArray<FSolverReal> MGroupLocalDampings;
	TArrayCollectionArray<bool> MGroupUseCCDs;

	TArray<TFunction<void(FSolverParticles&, const FSolverReal)>> MConstraintInits;
	TPBDActiveView<TArray<TFunction<void(FSolverParticles&, const FSolverReal)>>> MConstraintInitsActiveView;
	TArray<TFunction<void(FSolverParticles&, const FSolverReal)>> MConstraintRules;
	TPBDActiveView<TArray<TFunction<void(FSolverParticles&, const FSolverReal)>>> MConstraintRulesActiveView;
	TArray<TFunction<void(FSolverParticles&, const FSolverReal)>> MPostCollisionConstraintRules;
	TPBDActiveView<TArray<TFunction<void(FSolverParticles&, const FSolverReal)>>> MPostCollisionConstraintRulesActiveView;

	TFunction<void(FSolverParticles&, const FSolverReal, const FSolverReal, const int32)> MKinematicUpdate;
	TFunction<void(FSolverCollisionParticles&, const FSolverReal, const FSolverReal, const int32)> MCollisionKinematicUpdate;
	//Newton specific lambdas:
	TFunction<void(const bool)> UpdatePositionBasedState;
	TFunction<void(const PMatrix<FSolverReal, 3, 3>&, PMatrix<FSolverReal, 3, 3>&, const int32)> PStress;
	TFunction<void(const PMatrix<FSolverReal, 3, 3>&, const PMatrix<FSolverReal, 3, 3>&, PMatrix<FSolverReal, 3, 3>&, const int32)> DeltaPStress;
	TFunction<void(TArray<FSolverVec3>&)> ProjectBCs;
	TFunction<void(const FSolverReal, TArray<FSolverVec3>&)> SetBCs;
	TFunction<void(const FSolverReal, const FSolverReal, const TArray<FSolverReal>&, bool, TArray<FSolverVec3>&)> MAddExternalForce;

	int32 MNumNewtonIterations;
	int32 MNumCGIterations;
	TArray<TVector<int32, 4>> MMesh;
	TArray<TArray<TVector<int32, 2>>> MIncidentElements;
	TArray<FSolverReal> Measure;
	//Material Constants:
	FSolverReal Mu;
	FSolverReal Lambda;

	FSolverVec3 MGravity;
	FSolverReal MCollisionThickness;
	FSolverReal MCoefficientOfFriction;
	FSolverReal MDamping;
	FSolverReal MLocalDamping;
	FSolverReal MTime;
	FSolverReal MSmoothDt;
	FSolverReal MNewtonTol;
	FSolverReal MCGTol;
	TArray<FSolverVec3> ElementForces;
	TArray<FSolverVec3> MVn;
	TArray<FSolverReal> MNodalMass;
	TArray<FSolverReal> MResidualNorm;
	TArray<int32> MConstrainedVertices;
	TArray<FSolverVec3> MBCPositions;
	bool NoVerbose = false;
	ConsModelCaches ConsCaches;
	TUniquePtr<ElasticFEM<FSolverReal, FSolverParticles>> MEFem;
	TUniquePtr<TArray<int32>> use_list;
	bool bWriteDebugInfo;
};


}  // End namespace Chaos::Softs
//
//#if !defined(CHAOS_POST_ITERATION_UPDATES_ISPC_ENABLED)
//#define CHAOS_POST_ITERATION_UPDATES_ISPC_ENABLED 1
//#endif


