// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"


namespace Chaos
{
	class FAABBVectorized;

	namespace Private
	{
		class FOBBVectorized
		{
		public:
			FOBBVectorized(const FRigidTransform3& Transform, const FVec3f& HalfExtentsIn, const FVec3f& InvScale);
			bool IntersectAABB(const FAABBVectorized& Bounds) const;

		private:
			VectorRegister4Float Position;
			VectorRegister4Float XAxis;
			VectorRegister4Float YAxis;
			VectorRegister4Float ZAxis;
			VectorRegister4Float HalfExtents;

			VectorRegister4Float MaxObb;
			VectorRegister4Float MinObb;
			VectorRegister4Float InvScale;
		};
	}
}