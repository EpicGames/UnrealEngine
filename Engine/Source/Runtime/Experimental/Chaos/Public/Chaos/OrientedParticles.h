// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/ArrayCollectionArray.h"
#include "Chaos/Particles.h"
#include "Chaos/Rotation.h"

namespace Chaos
{
template<class T, int d>
class TOrientedParticles : public TParticles<T, d>
{
  public:
	TOrientedParticles()
	    : TParticles<T, d>()
	{
		TArrayCollection::AddArray(&MR);
	}
	TOrientedParticles(const TOrientedParticles<T, d>& Other) = delete;
	TOrientedParticles(TOrientedParticles<T, d>&& Other)
	    : TParticles<T, d>(MoveTemp(Other)), MR(MoveTemp(Other.MR))
	{
		TArrayCollection::AddArray(&MR);
	}

	const TRotation<T, d>& R(const int32 Index) const { return MR[Index]; }
	TRotation<T, d>& R(const int32 Index) { return MR[Index]; }

  private:
	TArrayCollectionArray<TRotation<T, d>> MR;
};
}
