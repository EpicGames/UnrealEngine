// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Algo/BinarySearch.h"
#include "Chaos/Framework/Parallel.h"
#include "Chaos/Vector.h"

namespace Chaos
{
	class FActiveViewRange final
	{
	public:
		FActiveViewRange()
			: bIsActive(0)
			, Offset(0)
		{}

		explicit FActiveViewRange(int32 InOffset, bool bActive = false)
			: bIsActive(bActive ? 1 : 0)
			, Offset((uint32)InOffset)
		{}

		bool IsActive() const
		{
			return !!bIsActive && Offset > 0;
		}

		int32 GetOffset() const
		{
			return (int32)Offset;
		}

		void SetActive(bool bActivate)
		{
			bIsActive = bActivate;
		}

	private:
		uint32 bIsActive : 1;
		uint32 Offset : 31;
	};

	// Index based view, specialized for working with several ranges within a same array such as particles.
	template<typename TItemsType>
	class TPBDActiveView
	{
	public:
		TPBDActiveView(TItemsType& InItems)
			: Items(InItems)
		{}

		// Return all items, including those not in the view.
		TItemsType& GetItems() const { return Items; }

		// Add a new active (or inactive) range at the end of the list, and return its offset.
		int32 AddRange(int32 NumItems, bool bActivate = true);

		// Return the number of items in the range starting at the specified offset.
		int32 GetRangeSize(int32 Offset) const;

		// Activate (or deactivate) the range starting at the specified offset.
		void ActivateRange(int32 Offset, bool bActivate);

		// Execute the specified function on all active items.
		void SequentialFor(TFunctionRef<void(TItemsType&, int32)> Function) const;

		// Execute the specified function in parallel, on all items for each active range (sequential range, parallel items). Set MinParallelSize to run sequential on the smaller ranges.
		void ParallelFor(TFunctionRef<void(TItemsType&, int32)> Function, int32 MinParallelSize = TNumericLimits<int32>::Max()) const;

		// Execute the specified function in nested parallel for loops, on all items for each active range (parallel range, parallel items). Set MinParallelSize to run sequential on the smaller ranges.
		void ParallelFor(TFunctionRef<void(TItemsType&, int32)> Function, bool bForceSingleThreadedRange, int32 MinParallelSize = TNumericLimits<int32>::Max()) const;

		// Execute the specified function in sequence for all active range. Callee responsible for inner loop.
		void RangeFor(TFunctionRef<void(TItemsType&, int32, int32)> Function) const;

		// Execute the specified function in parallel for all active ranges. Callee responsible for inner loop.
		void RangeFor(TFunctionRef<void(TItemsType&, int32, int32)> Function, bool bForceSingleThreadedRange) const;

		// Remove all ranges above the current given size.
		void Reset(int32 Offset = 0);

		// Return whether there is any active range in the view.
		bool HasActiveRange() const;

		// Return the total number of active items from all active ranges.
		int32 GetActiveSize() const;

		// Return a list of pair (offset, range) of all active ranges.
		TArray<TVector<int32, 2>, TInlineAllocator<8>> GetActiveRanges() const;

		// Return internal ranges.
		TConstArrayView<FActiveViewRange> GetAllRanges() const { return Ranges; }
		UE_DEPRECATED(5.6, "This method has been deprecated as the underlying type of Ranges has changed. Use GetAllRanges instead.")
		TConstArrayView<int32> GetRanges() const { return TConstArrayView<int32>(); }

		int32 GetNumRanges() const { return Ranges.Num(); }

	private:

		int32 GetRangeIndex(int32 Offset) const
		{
			// Binary search upper bound range of this offset
			const int32 Index = Algo::UpperBound(Ranges, FActiveViewRange(Offset), [](const FActiveViewRange A, const FActiveViewRange B) { return A.GetOffset() < B.GetOffset(); });
			check(Ranges.IsValidIndex(Index));
			return Index;
		}

		TItemsType& Items;
		TArray<FActiveViewRange> Ranges;
	};

	template <class TItemsType>
	int32 TPBDActiveView<TItemsType>::AddRange(int32 NumItems, bool bActivate)
	{
		const int32 Offset = Ranges.Num() ? Ranges.Last().GetOffset() : 0;
		if (NumItems)
		{
			const int32 Size = Offset + NumItems;
			Ranges.Emplace(Size, bActivate);
		}
		return Offset;
	}

	template <class TItemsType>
	int32 TPBDActiveView<TItemsType>::GetRangeSize(int32 Offset) const
	{
		const int32 Index = GetRangeIndex(Offset);
		// Return size regardless or activation state
		return Ranges[Index].GetOffset() - Offset;
	}

	template <class TItemsType>
	void TPBDActiveView<TItemsType>::ActivateRange(int32 Offset, bool bActivate)
	{
		const int32 Index = GetRangeIndex(Offset);
		Ranges[Index].SetActive(bActivate);
	}

	template <class TItemsType>
	void TPBDActiveView<TItemsType>::SequentialFor(TFunctionRef<void(TItemsType&, int32)> Function) const
	{
		int32 Offset = 0;
		for (const FActiveViewRange Range : Ranges)
		{
			if (Range.IsActive())
			{
				// Active range
				for (int32 Index = Offset; Index < Range.GetOffset(); ++Index)
				{
					Function(Items, Index);
				}
			}
			Offset = Range.GetOffset();
		}
	}

	template <class TItemsType>
	void TPBDActiveView<TItemsType>::ParallelFor(TFunctionRef<void(TItemsType&, int32)> Function, int32 MinParallelBatchSize) const
	{
		int32 Offset = 0;
		for (const FActiveViewRange Range : Ranges)
		{
			if (Range.IsActive())
			{
				// Active range
				const int32 RangeSize = Range.GetOffset() - Offset;
				PhysicsParallelFor(RangeSize, [this, Offset, &Function](int32 Index)
				{
					Function(Items, Offset + Index);
				}, /*bForceSingleThreaded =*/ RangeSize < MinParallelBatchSize);
			}
			Offset = Range.GetOffset();
		}
	}

	template <class TItemsType>
	void TPBDActiveView<TItemsType>::ParallelFor(TFunctionRef<void(TItemsType&, int32)> Function, bool bForceSingleThreadedRange, int32 MinParallelBatchSize) const
	{
		const TArray<TVector<int32, 2>, TInlineAllocator<8>> ActiveRanges = GetActiveRanges();

		PhysicsParallelFor(ActiveRanges.Num(), [this, &Function, &ActiveRanges, &MinParallelBatchSize](int32 RangeIndex)
		{
			const int32 Offset = ActiveRanges[RangeIndex][0];
			const int32 RangeSize = ActiveRanges[RangeIndex][1] - Offset;
			PhysicsParallelFor(RangeSize, [this, Offset, &Function](int32 Index)
			{
				Function(Items, Offset + Index);
			}, /*bForceSingleThreaded =*/ RangeSize < MinParallelBatchSize);
		}, bForceSingleThreadedRange);
	}

	template <class TItemsType>
	void TPBDActiveView<TItemsType>::RangeFor(TFunctionRef<void(TItemsType&, int32, int32)> Function) const
	{
		int32 Offset = 0;
		for (const FActiveViewRange Range : Ranges)
		{
			if (Range.IsActive())
			{
				// Active Range
				Function(Items, Offset, Range.GetOffset());
			}
			Offset = Range.GetOffset();
		}
	}

	template <class TItemsType>
	void TPBDActiveView<TItemsType>::RangeFor(TFunctionRef<void(TItemsType&, int32, int32)> Function, bool bForceSingleThreadedRange) const
	{
		const TArray<TVector<int32, 2>, TInlineAllocator<8>> ActiveRanges = GetActiveRanges();

		PhysicsParallelFor(ActiveRanges.Num(), [this, &Function, &ActiveRanges](int32 RangeIndex)
		{
			const TVector<int32, 2>& ActiveRange = ActiveRanges[RangeIndex];
			const int32 Offset = ActiveRange[0];
			const int32 Range = ActiveRange[1];
			Function(Items, Offset, Range);
		}, bForceSingleThreadedRange);
	}

	template <class TItemsType>
	void TPBDActiveView<TItemsType>::Reset(int32 Offset)
	{
		for (int32 Index = 0; Index < Ranges.Num(); ++Index)
		{
			if (Ranges[Index].GetOffset() > Offset)
			{
				Ranges.SetNum(Index);
				break;
			}
		}
	}

	template <class TItemsType>
	bool TPBDActiveView<TItemsType>::HasActiveRange() const
	{
		for (const FActiveViewRange Range : Ranges)
		{
			if (Range.IsActive())
			{
				return true;
			}
		}
		return false;
	}

	template <class TItemsType>
	int32 TPBDActiveView<TItemsType>::GetActiveSize() const
	{
		int32 ActiveSize = 0;
		int32 Offset = 0;
		for (const FActiveViewRange Range : Ranges)
		{
			if (Range.IsActive())
			{
				ActiveSize += Range.GetOffset() - Offset;
			}
			Offset = Range.GetOffset();
		}
		return ActiveSize;
	}

	template <class TItemsType>
	TArray<TVector<int32, 2>, TInlineAllocator<8>> TPBDActiveView<TItemsType>::GetActiveRanges() const
	{
		TArray<TVector<int32, 2>, TInlineAllocator<8>> ActiveRanges;

		int32 Offset = 0;
		for (const FActiveViewRange Range : Ranges)
		{
			if (Range.IsActive())
			{
				ActiveRanges.Add(TVector<int32, 2>(Offset, Range.GetOffset()));
			}
			Offset = Range.GetOffset();
		}
		return ActiveRanges;
	}
}
