// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/PBDSpringConstraintsBase.h"
#include "ChaosStats.h"
#include "Chaos/GraphColoring.h"
#include "Chaos/Framework/Parallel.h"


namespace Chaos::Softs
{

	class FPBDAltitudeSpringConstraints
	{
	public:
		FPBDAltitudeSpringConstraints(const FSolverParticles& InParticles, TArray<TVec4<int32>>&& InConstraints, const FSolverReal InStiffness = (FSolverReal)1.)
			: Constraints(InConstraints), Stiffness(InStiffness)
		{	
			
			RestLength.Init((FSolverReal)0., 4 * Constraints.Num());
			Volumes.Init((FSolverReal)0., Constraints.Num());
			for (int32 ElementIndex = 0; ElementIndex < Constraints.Num(); ElementIndex++)
			{	
				TVec4<int32> Constraint = Constraints[ElementIndex];
				for (int32 i = 0; i < 4; i++)
				{
					const FSolverVec3& P1 = InParticles.GetX(Constraint[i]);
					const FSolverVec3& P2 = InParticles.GetX(Constraint[(i + 1) % 4]);
					const FSolverVec3& P3 = InParticles.GetX(Constraint[(i + 2) % 4]);
					const FSolverVec3& P4 = InParticles.GetX(Constraint[(i + 3) % 4]);
					RestLength[4* ElementIndex +i] = FSolverVec3::DotProduct(FSolverVec3::CrossProduct(P3 - P2, P4 - P2), P1 - P2) / FSolverVec3::CrossProduct(P3 - P2, P4 - P2).Size();

				}
				const FSolverVec3& P1 = InParticles.GetX(Constraint[0]);
				const FSolverVec3& P2 = InParticles.GetX(Constraint[1]);
				const FSolverVec3& P3 = InParticles.GetX(Constraint[2]);
				const FSolverVec3& P4 = InParticles.GetX(Constraint[3]);
				Volumes[ElementIndex] = FSolverVec3::DotProduct(FSolverVec3::CrossProduct(P2 - P1, P3 - P1), P4 - P1) / (FSolverReal)6.;
			}
			
			InitColor(InParticles);
		}
		virtual ~FPBDAltitudeSpringConstraints() = default;

		TVec4<FSolverVec3> GetGradients(const FSolverParticles& InParticles, const int32 ElementIndex, const int32 ie) const
		{
			TVec4<FSolverVec3> Grads(FSolverVec3((FSolverReal)0.));;
			const TVec4<int32>& Constraint = Constraints[ElementIndex];
			const int32 i1 = Constraint[ie];
			const int32 i2 = Constraint[(ie + 1) % 4];
			const int32 i3 = Constraint[(ie + 2) % 4];
			const int32 i4 = Constraint[(ie + 3) % 4];
			const FSolverVec3& P1 = InParticles.P(i1);
			const FSolverVec3& P2 = InParticles.P(i2);
			const FSolverVec3& P3 = InParticles.P(i3);
			const FSolverVec3& P4 = InParticles.P(i4);
			const FSolverVec3 P1P2 = P1 - P2;
			const FSolverVec3 P3P2 = P3 - P2;
			const FSolverVec3 P4P2 = P4 - P2;

			FSolverVec3 ScaledBottomNormal = FSolverVec3::CrossProduct(P3 - P2, P4 - P2);
			FSolverReal ScaledVolume = FSolverVec3::DotProduct(ScaledBottomNormal, P1 - P2);
			FSolverReal ScaledArea = ScaledBottomNormal.Size();
			TVec4<FSolverVec3> dAdX(FSolverVec3((FSolverReal)0.));
			FSolverVec3 ScaleddAdV((FSolverReal)0.);
			ScaleddAdV[0] = -ScaledBottomNormal[1] * P4P2[2] + ScaledBottomNormal[2] * P4P2[1];
			ScaleddAdV[1] = ScaledBottomNormal[0] * P4P2[2] - ScaledBottomNormal[2] * P4P2[0];
			ScaleddAdV[2] = -ScaledBottomNormal[0] * P4P2[1] + ScaledBottomNormal[1] * P4P2[0];
			FSolverVec3 ScaleddAdW((FSolverReal)0.);
			ScaleddAdW[0] = ScaledBottomNormal[1] * P3P2[2] - ScaledBottomNormal[2] * P3P2[1];
			ScaleddAdW[1] = -ScaledBottomNormal[0] * P3P2[2] + ScaledBottomNormal[2] * P3P2[0];
			ScaleddAdW[2] = ScaledBottomNormal[0] * P3P2[1] - ScaledBottomNormal[1] * P3P2[0];
			dAdX[2] = (FSolverReal)1. / ScaledArea * ScaleddAdV;
			dAdX[3] = (FSolverReal)1. / ScaledArea * ScaleddAdW;
			dAdX[1] = -dAdX[2] - dAdX[3];

			TVec4<FSolverVec3> dVdX(FSolverVec3((FSolverReal)0.));
			dVdX[0] = FSolverVec3::CrossProduct(P3P2, P4P2);
			dVdX[2] = FSolverVec3::CrossProduct(P4P2, P1P2);
			dVdX[3] = FSolverVec3::CrossProduct(P1P2, P3P2);
			dVdX[1] = -(dVdX[0] + dVdX[2] + dVdX[3]);
			for (int32 je = 0; je < 4; je++)
			{
				Grads[je] = -ScaledVolume / (ScaledArea * ScaledArea) * dAdX[je] + (FSolverReal)1. / ScaledArea * dVdX[je];
			}


			return Grads;
		}

		FSolverReal GetScalingFactor(const FSolverParticles& InParticles, const int32 ElementIndex, const int32 ie, const TVec4<FSolverVec3>& Grads) const
		{
			const TVec4<int32>& Constraint = Constraints[ElementIndex];
			const int32 i1 = Constraint[ie];
			const int32 i2 = Constraint[(ie + 1) % 4];
			const int32 i3 = Constraint[(ie + 2) % 4];
			const int32 i4 = Constraint[(ie + 3) % 4];
			const FSolverVec3& P1 = InParticles.P(i1);
			const FSolverVec3& P2 = InParticles.P(i2);
			const FSolverVec3& P3 = InParticles.P(i3);
			const FSolverVec3& P4 = InParticles.P(i4);
			FSolverVec3 ScaledBottomNormal = FSolverVec3::CrossProduct(P3 - P2, P4 - P2);
			FSolverReal ScaledVolume = FSolverVec3::DotProduct(ScaledBottomNormal, P1 - P2);
			FSolverReal CurrentLength = ScaledVolume / ScaledBottomNormal.Size();
			const FSolverReal S = (CurrentLength - RestLength[4*ElementIndex+ie]) / (
				InParticles.InvM(i1) * Grads[0].SizeSquared() +
				InParticles.InvM(i2) * Grads[1].SizeSquared() +
				InParticles.InvM(i3) * Grads[2].SizeSquared() +
				InParticles.InvM(i4) * Grads[3].SizeSquared());
			return Stiffness * S;
		}

		virtual void ApplySingleElement(FSolverParticles& Particles, const FSolverReal Dt, const int32 ElementIndex) const
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(STAT_ChaosPBDAltitudeSpringApplySingle);
			const TVec4<int32>& Constraint = Constraints[ElementIndex];
			for (int ie = 0; ie < 4; ++ie)
			{
				
				const int32 i1 = Constraint[ie];
				const int32 i2 = Constraint[(ie + 1) % 4];
				const int32 i3 = Constraint[(ie + 2) % 4];
				const int32 i4 = Constraint[(ie + 3) % 4];
				const TVec4<FSolverVec3> Grads = GetGradients(Particles, ElementIndex, ie);
				const FSolverReal S = GetScalingFactor(Particles, ElementIndex, ie, Grads);
				Particles.P(i1) -= S * Particles.InvM(i1) * Grads[0];
				Particles.P(i2) -= S * Particles.InvM(i2) * Grads[1];
				Particles.P(i3) -= S * Particles.InvM(i3) * Grads[2];
				Particles.P(i4) -= S * Particles.InvM(i4) * Grads[3];
			}
		}

		virtual void ApplySingleElementAllInOne(FSolverParticles& Particles, const FSolverReal Dt, const int32 ElementIndex) const
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(STAT_ChaosPBDAltitudeSpringApplySingle);
			const TVec4<int32>& Constraint = Constraints[ElementIndex];
			for (int ie = 0; ie < 4; ++ie)
			{
				TVec4<FSolverVec3> Grads(FSolverVec3((FSolverReal)0.));
				const int32 i1 = Constraint[ie];
				const int32 i2 = Constraint[(ie + 1) % 4];
				const int32 i3 = Constraint[(ie + 2) % 4];
				const int32 i4 = Constraint[(ie + 3) % 4];
				const FSolverVec3& P1 = Particles.P(i1);
				const FSolverVec3& P2 = Particles.P(i2);
				const FSolverVec3& P3 = Particles.P(i3);
				const FSolverVec3& P4 = Particles.P(i4);
				const FSolverVec3 P1P2 = P1 - P2;
				const FSolverVec3 P3P2 = P3 - P2;
				const FSolverVec3 P4P2 = P4 - P2;

				FSolverVec3 ScaledBottomNormal = FSolverVec3::CrossProduct(P3 - P2, P4 - P2);
				FSolverReal ScaledVolume = FSolverVec3::DotProduct(ScaledBottomNormal, P1 - P2);
				FSolverReal ScaledArea = ScaledBottomNormal.Size();
				TVec4<FSolverVec3> dAdX(FSolverVec3((FSolverReal)0.));
				FSolverVec3 ScaleddAdV((FSolverReal)0.);
				ScaleddAdV[0] = -ScaledBottomNormal[1] * P4P2[2] + ScaledBottomNormal[2] * P4P2[1];
				ScaleddAdV[1] = ScaledBottomNormal[0] * P4P2[2] - ScaledBottomNormal[2] * P4P2[0];
				ScaleddAdV[2] = -ScaledBottomNormal[0] * P4P2[1] + ScaledBottomNormal[1] * P4P2[0];
				FSolverVec3 ScaleddAdW((FSolverReal)0.);
				ScaleddAdW[0] = ScaledBottomNormal[1] * P3P2[2] - ScaledBottomNormal[2] * P3P2[1];
				ScaleddAdW[1] = -ScaledBottomNormal[0] * P3P2[2] + ScaledBottomNormal[2] * P3P2[0];
				ScaleddAdW[2] = ScaledBottomNormal[0] * P3P2[1] - ScaledBottomNormal[1] * P3P2[0];
				dAdX[2] = (FSolverReal)1. / ScaledArea * ScaleddAdV;
				dAdX[3] = (FSolverReal)1. / ScaledArea * ScaleddAdW;
				dAdX[1] = -dAdX[2] - dAdX[3];

				TVec4<FSolverVec3> dVdX(FSolverVec3((FSolverReal)0.));
				dVdX[0] = FSolverVec3::CrossProduct(P3P2, P4P2);
				dVdX[2] = FSolverVec3::CrossProduct(P4P2, P1P2);
				dVdX[3] = FSolverVec3::CrossProduct(P1P2, P3P2);
				dVdX[1] = -(dVdX[0] + dVdX[2] + dVdX[3]);
				for (int32 je = 0; je < 4; je++)
				{
					Grads[je] = -ScaledVolume / (ScaledArea * ScaledArea) * dAdX[je] + (FSolverReal)1. / ScaledArea * dVdX[je];
				}
				FSolverReal CurrentLength = ScaledVolume / ScaledBottomNormal.Size();
				const FSolverReal S = Stiffness * (CurrentLength - RestLength[4 * ElementIndex + ie]) / (
					Particles.InvM(i1) * Grads[0].SizeSquared() +
					Particles.InvM(i2) * Grads[1].SizeSquared() +
					Particles.InvM(i3) * Grads[2].SizeSquared() +
					Particles.InvM(i4) * Grads[3].SizeSquared());
				Particles.P(i1) -= S * Particles.InvM(i1) * Grads[0];
				Particles.P(i2) -= S * Particles.InvM(i2) * Grads[1];
				Particles.P(i3) -= S * Particles.InvM(i3) * Grads[2];
				Particles.P(i4) -= S * Particles.InvM(i4) * Grads[3];
			}
		}

		virtual void ApplySingleElementVolumeAllInOne(FSolverParticles& Particles, const FSolverReal Dt, const int32 ElementIndex) const
		{
			TVec4<FSolverVec3> Grads;
			const TVec4<int32>& Constraint = Constraints[ElementIndex];
			const FSolverVec3& P1 = Particles.P(Constraint[0]);
			const FSolverVec3& P2 = Particles.P(Constraint[1]);
			const FSolverVec3& P3 = Particles.P(Constraint[2]);
			const FSolverVec3& P4 = Particles.P(Constraint[3]);
			const FSolverVec3 P2P1 = P2 - P1;
			const FSolverVec3 P4P1 = P4 - P1;
			const FSolverVec3 P3P1 = P3 - P1;
			Grads[1] = FSolverVec3::CrossProduct(P3P1, P4P1) / (FSolverReal)6.;
			Grads[2] = FSolverVec3::CrossProduct(P4P1, P2P1) / (FSolverReal)6.;
			Grads[3] = FSolverVec3::CrossProduct(P2P1, P3P1) / (FSolverReal)6.;
			Grads[0] = -(Grads[1] + Grads[2] + Grads[3]);
			const int32 i1 = Constraint[0];
			const int32 i2 = Constraint[1];
			const int32 i3 = Constraint[2];
			const int32 i4 = Constraint[3];

			const FSolverReal Volume = FSolverVec3::DotProduct(FSolverVec3::CrossProduct(P2 - P1, P3 - P1), P4 - P1) / (FSolverReal)6.;
			const FSolverReal S = Stiffness*(Volume - Volumes[ElementIndex]) / (
				Particles.InvM(i1) * Grads[0].SizeSquared() +
				Particles.InvM(i2) * Grads[1].SizeSquared() +
				Particles.InvM(i3) * Grads[2].SizeSquared() +
				Particles.InvM(i4) * Grads[3].SizeSquared());
			
			Particles.P(i1) -= S * Particles.InvM(i1) * Grads[0];
			Particles.P(i2) -= S * Particles.InvM(i2) * Grads[1];
			Particles.P(i3) -= S * Particles.InvM(i3) * Grads[2];
			Particles.P(i4) -= S * Particles.InvM(i4) * Grads[3];
		}

		virtual void ApplySingleElementShortest(FSolverParticles& Particles, const FSolverReal Dt, const int32 ElementIndex) const
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(STAT_ChaosPBDAltitudeSpringApplySingleShortest);
			const TVec4<int32>& Constraint = Constraints[ElementIndex];
			int32 ie = GetSingleElementShortestLength(Particles, Dt, ElementIndex);
			
			const int32 i1 = Constraint[ie];
			const int32 i2 = Constraint[(ie + 1) % 4];
			const int32 i3 = Constraint[(ie + 2) % 4];
			const int32 i4 = Constraint[(ie + 3) % 4];
			const TVec4<FSolverVec3> Grads = GetGradients(Particles, ElementIndex, ie);
			const FSolverReal S = GetScalingFactor(Particles, ElementIndex, ie, Grads);
			Particles.P(i1) -= S * Particles.InvM(i1) * Grads[0];
			Particles.P(i2) -= S * Particles.InvM(i2) * Grads[1];
			Particles.P(i3) -= S * Particles.InvM(i3) * Grads[2];
			Particles.P(i4) -= S * Particles.InvM(i4) * Grads[3];

		}

		int32 GetSingleElementShortestLength(FSolverParticles& Particles, const FSolverReal Dt, const int32 ElementIndex) const 
		{	
			int32 ReturnIndex = -1;
			FSolverReal MinConstraint = TNumericLimits<FSolverReal>::Max();
			const TVec4<int32>& Constraint = Constraints[ElementIndex];
			for (int ie = 0; ie < 4; ++ie)
			{
				const int32 i1 = Constraint[ie];
				const int32 i2 = Constraint[(ie + 1) % 4];
				const int32 i3 = Constraint[(ie + 2) % 4];
				const int32 i4 = Constraint[(ie + 3) % 4];
				const FSolverVec3& P1 = Particles.P(i1);
				const FSolverVec3& P2 = Particles.P(i2);
				const FSolverVec3& P3 = Particles.P(i3);
				const FSolverVec3& P4 = Particles.P(i4);
				FSolverVec3 ScaledBottomNormal = FSolverVec3::CrossProduct(P3 - P2, P4 - P2);
				FSolverReal ScaledVolume = FSolverVec3::DotProduct(ScaledBottomNormal, P1 - P2);
				FSolverReal CurrentLength = ScaledVolume / ScaledBottomNormal.Size();
				FSolverReal CurrentConstraint = CurrentLength - RestLength[4 * ElementIndex + ie];
				if (CurrentConstraint < MinConstraint)
				{
					ReturnIndex = ie;
					MinConstraint = CurrentConstraint;
				}
			}

			//another stratey: find the face with largest area
			/*FSolverReal MaxConstraint = (FSolverReal)-1.;
			for (int ie = 0; ie < 4; ++ie)
			{
				const int32 i1 = Constraint[ie];
				const int32 i2 = Constraint[(ie + 1) % 4];
				const int32 i3 = Constraint[(ie + 2) % 4];
				const int32 i4 = Constraint[(ie + 3) % 4];
				const FSolverVec3& P1 = Particles.P(i1);
				const FSolverVec3& P2 = Particles.P(i2);
				const FSolverVec3& P3 = Particles.P(i3);
				const FSolverVec3& P4 = Particles.P(i4);
				FSolverReal ScaledBottomNormalArea = FSolverVec3::CrossProduct(P3 - P2, P4 - P2).Size();
				if (ScaledBottomNormalArea > MaxConstraint)
				{
					ReturnIndex = ie;
					MaxConstraint = ScaledBottomNormalArea;
				}
			}*/

			ensure(ReturnIndex != -1);
			return ReturnIndex;

		}

		void ApplyInSerial(FSolverParticles& Particles, const FSolverReal Dt) const
		{
			//SCOPE_CYCLE_COUNTER(STAT_ChaosXPBDCorotated);
			TRACE_CPUPROFILER_EVENT_SCOPE(STAT_ChaosPBDAltitudeSpringApplySerial);
			for (int32 ElementIndex = 0; ElementIndex < Constraints.Num(); ++ElementIndex)
			{
				ApplySingleElement(Particles, Dt, ElementIndex);
				//ApplySingleElementShortest(Particles, Dt, ElementIndex);
				//ApplySingleElementVolumeAllInOne(Particles, Dt, ElementIndex);
			}
		}

		void ApplyInParallel(FSolverParticles& Particles, const FSolverReal Dt) const
		{
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(STAT_ChaosPBDAltitudeSpringApply);
				if ((ConstraintsPerColorStartIndex.Num() > 1))//&& (MeshConstraints.Num() > Chaos_Spring_ParallelConstraintCount))
				{
					const int32 ConstraintColorNum = ConstraintsPerColorStartIndex.Num() - 1;

					for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
					{
						const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
						const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
						PhysicsParallelFor(ColorSize, [&](const int32 Index)
							{
								const int32 ConstraintIndex = ColorStart + Index;
								ApplySingleElement(Particles, Dt, ConstraintIndex); 
								//ApplySingleElementShortest(Particles, Dt, ConstraintIndex);
								//ApplySingleElementVolumeAllInOne(Particles, Dt, ConstraintIndex);
								//ApplySingleElementAllInOne(Particles, Dt, ConstraintIndex);
							});
					}
				}
			}
		}

	protected:
		TArray<TVec4<int32>> Constraints;

	protected:


	private:
		void InitColor(const FSolverParticles& Particles)
		{

			{
				const TArray<TArray<int32>> ConstraintsPerColor = FGraphColoring::ComputeGraphColoring(Constraints, Particles);

				// Reorder constraints based on color so each array in ConstraintsPerColor contains contiguous elements.
				TArray<TVec4<int32>> ReorderedConstraints;
				TArray<FSolverReal> ReorderedRestLength;
				TArray<FSolverReal> ReorderedVolumes;
				TArray<int32> OrigToReorderedIndices; // used to reorder stiffness indices
				ReorderedConstraints.SetNumUninitialized(Constraints.Num());
				ReorderedRestLength.SetNumUninitialized(RestLength.Num());
				ReorderedVolumes.SetNumUninitialized(Volumes.Num());
				OrigToReorderedIndices.SetNumUninitialized(Constraints.Num());

				ConstraintsPerColorStartIndex.Reset(ConstraintsPerColor.Num() + 1);

				int32 ReorderedIndex = 0;
				for (const TArray<int32>& ConstraintsBatch : ConstraintsPerColor)
				{
					ConstraintsPerColorStartIndex.Add(ReorderedIndex);
					for (const int32& BatchConstraint : ConstraintsBatch)
					{
						const int32 OrigIndex = BatchConstraint;
						ReorderedConstraints[ReorderedIndex] = Constraints[OrigIndex];
						//ReorderedMeasure[ReorderedIndex] = Measure[OrigIndex];
						for (int32 kk = 0; kk < 4; kk++)
						{
							ReorderedRestLength[4 * ReorderedIndex + kk] = RestLength[4 * OrigIndex + kk];
						}
						ReorderedVolumes[ReorderedIndex] = Volumes[OrigIndex];
						OrigToReorderedIndices[OrigIndex] = ReorderedIndex;

						++ReorderedIndex;
					}
				}
				ConstraintsPerColorStartIndex.Add(ReorderedIndex);

				Constraints = MoveTemp(ReorderedConstraints);
				RestLength = MoveTemp(ReorderedRestLength);
				Volumes = MoveTemp(ReorderedVolumes);
			}
		}

		TArray<FSolverReal> RestLength;
		TArray<FSolverReal> Volumes;
		FSolverReal Stiffness;
		TArray<int32> ConstraintsPerColorStartIndex;
	};

}  // End namespace Chaos::Softs
