// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// HEADER_UNIT_SKIP - Internal

#include "Chaos/Framework/Parallel.h"
#include "Chaos/PBDSoftsEvolutionFwd.h"
#include "Chaos/PBDSoftsSolverParticles.h"
#include "Chaos/SoftsSolverParticlesRange.h"
#include "Chaos/PBDStiffness.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "ChaosStats.h"

DECLARE_CYCLE_STAT(TEXT("Chaos PBD Anim Drive Constraint"), STAT_PBD_AnimDriveConstraint, STATGROUP_Chaos);

namespace Chaos::Softs
{

	class FPBDAnimDriveConstraint final
	{
	public:
		static bool IsEnabled(const FCollectionPropertyConstFacade& PropertyCollection)
		{
			return IsAnimDriveStiffnessEnabled(PropertyCollection, false);
		}

		FPBDAnimDriveConstraint(
			const int32 InParticleOffset,
			const int32 InParticleCount,
			const TArray<FSolverVec3>& InAnimationPositions,  // Use global indexation (will need adding ParticleOffset)
			const TArray<FSolverVec3>& InAnimationVelocities,  // Use global indexation (will need adding ParticleOffset)
			const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
			const FCollectionPropertyConstFacade& PropertyCollection
		)
			: AnimationPositions(InAnimationPositions)
			, AnimationVelocities(InAnimationVelocities)
			, ParticleOffset(InParticleOffset)
			, ParticleCount(InParticleCount)
			, Stiffness(
				FSolverVec2(GetWeightedFloatAnimDriveStiffness(PropertyCollection, 1.f)),
				WeightMaps.FindRef(GetAnimDriveStiffnessString(PropertyCollection, AnimDriveStiffnessName.ToString())),
				InParticleCount)
			, Damping(
				FSolverVec2(GetWeightedFloatAnimDriveDamping(PropertyCollection, 1.f)),
				WeightMaps.FindRef(GetAnimDriveDampingString(PropertyCollection, AnimDriveDampingName.ToString())),
				InParticleCount)
			, AnimDriveStiffnessIndex(PropertyCollection)
			, AnimDriveDampingIndex(PropertyCollection)
		{
		}

		FPBDAnimDriveConstraint(
			const int32 InParticleOffset,
			const int32 InParticleCount,
			const TArray<FSolverVec3>& InAnimationPositions,  // Use global indexation (will need adding ParticleOffset)
			const TArray<FSolverVec3>& /*InOldAnimationPositions*/,  // deprecated
			const TArray<FSolverVec3>& InAnimationVelocities,  // Use global indexation (will need adding ParticleOffset)
			const TConstArrayView<FRealSingle>& StiffnessMultipliers,  // Use local indexation
			const TConstArrayView<FRealSingle>& DampingMultipliers  // Use local indexation
		)
			: AnimationPositions(InAnimationPositions)
			, AnimationVelocities(InAnimationVelocities)
			, ParticleOffset(InParticleOffset)
			, ParticleCount(InParticleCount)
			, Stiffness(FSolverVec2::UnitVector, StiffnessMultipliers, InParticleCount)
			, Damping(FSolverVec2::UnitVector, DampingMultipliers, InParticleCount)
			, AnimDriveStiffnessIndex(ForceInit)
			, AnimDriveDampingIndex(ForceInit)
		{
		}

		~FPBDAnimDriveConstraint() = default;

		// Return the stiffness input values used by the constraint
		FSolverVec2 GetStiffness() const { return Stiffness.GetWeightedValue(); }

		// Return the damping input values used by the constraint
		FSolverVec2 GetDamping() const { return Damping.GetWeightedValue(); }

		void SetProperties(
			const FCollectionPropertyConstFacade& PropertyCollection,
			const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps)
		{
			if (IsAnimDriveStiffnessMutable(PropertyCollection))
			{
				const FSolverVec2 WeightedValue(GetWeightedFloatAnimDriveStiffness(PropertyCollection));
				if (IsAnimDriveStiffnessStringDirty(PropertyCollection))
				{
					const FString& WeightMapName = GetAnimDriveStiffnessString(PropertyCollection);
					Stiffness = FPBDStiffness(WeightedValue, WeightMaps.FindRef(WeightMapName), ParticleCount);
				}
				else
				{
					Stiffness.SetWeightedValue(WeightedValue);
				}
			}
			if (IsAnimDriveDampingMutable(PropertyCollection))
			{
				const FSolverVec2 WeightedValue(GetWeightedFloatAnimDriveDamping(PropertyCollection));
				if (IsAnimDriveDampingStringDirty(PropertyCollection))
				{
					const FString& WeightMapName = GetAnimDriveDampingString(PropertyCollection);
					Damping = FPBDStiffness(WeightedValue, WeightMaps.FindRef(WeightMapName), ParticleCount);
				}
				else
				{
					Damping.SetWeightedValue(WeightedValue);
				}
			}
		}

		inline void SetProperties(const FSolverVec2& InStiffness, const FSolverVec2& InDamping)
		{
			Stiffness.SetWeightedValue(InStiffness);
			Damping.SetWeightedValue(InDamping);
		}

		// Set stiffness offset and range, as well as the simulation stiffness exponent
		inline void ApplyProperties(const FSolverReal Dt, const int32 NumIterations)
		{
			Stiffness.ApplyPBDValues(Dt, NumIterations);
			Damping.ApplyPBDValues(Dt, NumIterations);
		}

		TConstArrayView<FSolverVec3> GetConstArrayView(const FSolverParticles& Particles, const TArray<FSolverVec3>& Data) const
		{
			return TConstArrayView<FSolverVec3>(Data.GetData() + ParticleOffset, ParticleCount);
		}

		TConstArrayView<FSolverVec3> GetConstArrayView(const FSolverParticlesRange& Particles, const TArray<FSolverVec3>& Data) const
		{
			return Particles.GetConstArrayView(Data);
		}

		template<typename SolverParticlesOrRange>
		inline void Apply(SolverParticlesOrRange& InParticles, const FSolverReal Dt) const
		{
			SCOPE_CYCLE_COUNTER(STAT_PBD_AnimDriveConstraint);

			if (Stiffness.HasWeightMap())
			{
				if (Damping.HasWeightMap())
				{
					const TConstArrayView<FSolverVec3> AnimationPositionsView = GetConstArrayView(InParticles, AnimationPositions);
					const TConstArrayView<FSolverVec3> AnimationVelocitiesView = GetConstArrayView(InParticles, AnimationVelocities);
					PhysicsParallelFor(ParticleCount, [this, &InParticles, &Dt, &AnimationPositionsView, &AnimationVelocitiesView](int32 Index)  // TODO: profile needed for these parallel loop based on particle count
					{
						const FSolverReal ParticleStiffness = Stiffness[Index];
						const FSolverReal ParticleDamping = Damping[Index];
						ApplyHelper(InParticles, ParticleStiffness, ParticleDamping, Dt, Index, AnimationPositionsView[Index], AnimationVelocitiesView[Index]);
					});
				}
				else
				{
					const FSolverReal ParticleDamping = (FSolverReal)Damping;
					const TConstArrayView<FSolverVec3> AnimationPositionsView = GetConstArrayView(InParticles, AnimationPositions);
					const TConstArrayView<FSolverVec3> AnimationVelocitiesView = GetConstArrayView(InParticles, AnimationVelocities);
					PhysicsParallelFor(ParticleCount, [this, &InParticles, ParticleDamping, &Dt, &AnimationPositionsView, &AnimationVelocitiesView](int32 Index)
					{
						const FSolverReal ParticleStiffness = Stiffness[Index];
						ApplyHelper(InParticles, ParticleStiffness, ParticleDamping, Dt, Index, AnimationPositionsView[Index], AnimationVelocitiesView[Index]);
					});
				}
			}
			else
			{
				const FSolverReal ParticleStiffness = (FSolverReal)Stiffness;
				if (Damping.HasWeightMap())
				{
					const TConstArrayView<FSolverVec3> AnimationPositionsView = GetConstArrayView(InParticles, AnimationPositions);
					const TConstArrayView<FSolverVec3> AnimationVelocitiesView = GetConstArrayView(InParticles, AnimationVelocities);
					PhysicsParallelFor(ParticleCount, [this, &InParticles, &ParticleStiffness, &Dt, &AnimationPositionsView, &AnimationVelocitiesView](int32 Index)
					{
						const FSolverReal ParticleDamping = Damping[Index];
						ApplyHelper(InParticles, ParticleStiffness, ParticleDamping, Dt, Index, AnimationPositionsView[Index], AnimationVelocitiesView[Index]);
					});
				}
				else
				{
					const FSolverReal ParticleDamping = (FSolverReal)Damping;
					const TConstArrayView<FSolverVec3> AnimationPositionsView = GetConstArrayView(InParticles, AnimationPositions);
					const TConstArrayView<FSolverVec3> AnimationVelocitiesView = GetConstArrayView(InParticles, AnimationVelocities);
					PhysicsParallelFor(ParticleCount, [this, &InParticles, &ParticleStiffness, &ParticleDamping, &Dt, &AnimationPositionsView, &AnimationVelocitiesView](int32 Index)
					{
						ApplyHelper(InParticles, ParticleStiffness, ParticleDamping, Dt, Index, AnimationPositionsView[Index], AnimationVelocitiesView[Index]);
					});
				}
			}
		}

		UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(AnimDriveStiffness, float);
		UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(AnimDriveDamping, float);

	private:


		template<typename SolverParticlesOrRange>
		inline void ApplyHelper(SolverParticlesOrRange& Particles, const FSolverReal InStiffness, const FSolverReal InDamping, const FSolverReal Dt, const int32 Index, const FSolverVec3& AnimPosition, const FSolverVec3& AnimVelocity) const
		{
			const int32 ParticleIndex = ParticleOffset + Index;
			if (Particles.InvM(ParticleIndex) == (FSolverReal)0.)
			{
				return;
			}

			FSolverVec3& ParticlePosition = Particles.P(ParticleIndex);

			ParticlePosition -= InStiffness * (ParticlePosition - AnimPosition);

			const FSolverVec3 ParticleDisplacement = ParticlePosition - Particles.GetX(ParticleIndex);
			const FSolverVec3 AnimationDisplacement = (AnimVelocity) * Dt;
			const FSolverVec3 RelativeDisplacement = ParticleDisplacement - AnimationDisplacement;

			ParticlePosition -= InDamping * RelativeDisplacement;
		}

	private:
		const TArray<FSolverVec3>& AnimationPositions;  // Use global index (needs adding ParticleOffset)
		const TArray<FSolverVec3>& AnimationVelocities;  // Use global index (needs adding ParticleOffset).
		const int32 ParticleOffset;
		const int32 ParticleCount;

		FPBDStiffness Stiffness;
		FPBDStiffness Damping;

		UE_CHAOS_DECLARE_INDEXED_PROPERTYCOLLECTION_NAME(AnimDriveStiffness, float);
		UE_CHAOS_DECLARE_INDEXED_PROPERTYCOLLECTION_NAME(AnimDriveDamping, float);
	};

}  // End namespace Chaos::Softs
