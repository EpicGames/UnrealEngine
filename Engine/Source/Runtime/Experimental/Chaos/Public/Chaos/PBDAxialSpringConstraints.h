// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// HEADER_UNIT_SKIP - Internal

#include "Chaos/PBDAxialSpringConstraintsBase.h"
#include "Chaos/CollectionPropertyFacade.h"

namespace Chaos
{
	class FTriangleMesh;
}

namespace Chaos::Softs
{

class FPBDAxialSpringConstraints : public FPBDAxialSpringConstraintsBase
{
	typedef FPBDAxialSpringConstraintsBase Base;

public:
	FPBDAxialSpringConstraints(
		const FSolverParticlesRange& Particles,
		const TArray<TVec3<int32>>& InConstraints,
		const TConstArrayView<FRealSingle>& StiffnessMultipliers,
		const FSolverVec2& InStiffness,
		bool bTrimKinematicConstraints,
		bool bInitColor = true)
		: Base(
			Particles,
			InConstraints,
			StiffnessMultipliers,
			InStiffness,
			bTrimKinematicConstraints)
	{
		if (bInitColor)
		{
			InitColor(Particles);
		}
	}

	FPBDAxialSpringConstraints(
		const FSolverParticles& Particles,
		int32 InParticleOffset,
		int32 InParticleCount,
		const TArray<TVec3<int32>>& InConstraints,
		const TConstArrayView<FRealSingle>& StiffnessMultipliers,
		const FSolverVec2& InStiffness,
		bool bTrimKinematicConstraints,
		bool bInitColor = true)
		: Base(
			Particles,
			InParticleOffset,
			InParticleCount,
			InConstraints,
			StiffnessMultipliers,
			InStiffness,
			bTrimKinematicConstraints)
	{
		if (bInitColor)
		{
			InitColor(Particles);
		}
	}

	virtual ~FPBDAxialSpringConstraints() override = default;

	template<typename SolverParticlesOrRange>
	CHAOS_API void Apply(SolverParticlesOrRange& InParticles, const FSolverReal Dt) const;

protected:
	using Base::Constraints;
	using Base::Barys;
	using Base::Stiffness;
	using Base::ParticleOffset;
	using Base::ParticleCount;

	template<typename SolverParticlesOrRange>
	CHAOS_API TArray<int32> InitColor(const SolverParticlesOrRange& InParticles);

private:
	template<typename SolverParticlesOrRange>
	void ApplyHelper(SolverParticlesOrRange& InParticles, const FSolverReal Dt, const int32 ConstraintIndex, const FSolverReal ExpStiffnessValue) const;

	TArray<int32> ConstraintsPerColorStartIndex; // Constraints are ordered so each batch is contiguous. This is ColorNum + 1 length so it can be used as start and end.
};

class FPBDAreaSpringConstraints final : public FPBDAxialSpringConstraints
{
public:
	static bool IsEnabled(const FCollectionPropertyConstFacade& PropertyCollection)
	{
		return IsAreaSpringStiffnessEnabled(PropertyCollection, false);
	}

	FPBDAreaSpringConstraints(
		const FSolverParticlesRange& Particles,
		const TArray<TVec3<int32>>& InConstraints,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
		const FCollectionPropertyConstFacade& PropertyCollection,
		bool bTrimKinematicConstraints)
		: FPBDAxialSpringConstraints(
			Particles,
			InConstraints,
			WeightMaps.FindRef(GetAreaSpringStiffnessString(PropertyCollection, AreaSpringStiffnessName.ToString())),
			FSolverVec2(GetWeightedFloatAreaSpringStiffness(PropertyCollection, 1.f)),
			bTrimKinematicConstraints)
		, WarpScale(FSolverVec2(1.f))
		, WeftScale(FSolverVec2(1.f))
		, AreaSpringStiffnessIndex(PropertyCollection)
		, AreaSpringWarpScaleIndex(ForceInit)
		, AreaSpringWeftScaleIndex(ForceInit)
	{}

	FPBDAreaSpringConstraints(
		const FSolverParticles& Particles,
		int32 InParticleOffset,
		int32 InParticleCount,
		const TArray<TVec3<int32>>& InConstraints,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
		const FCollectionPropertyConstFacade& PropertyCollection,
		bool bTrimKinematicConstraints)
		: FPBDAxialSpringConstraints(
			Particles,
			InParticleOffset,
			InParticleCount,
			InConstraints,
			WeightMaps.FindRef(GetAreaSpringStiffnessString(PropertyCollection, AreaSpringStiffnessName.ToString())),
			FSolverVec2(GetWeightedFloatAreaSpringStiffness(PropertyCollection, 1.f)),
			bTrimKinematicConstraints)
		, WarpScale(FSolverVec2(1.f))
		, WeftScale(FSolverVec2(1.f))
		, AreaSpringStiffnessIndex(PropertyCollection)
		, AreaSpringWarpScaleIndex(ForceInit)
		, AreaSpringWeftScaleIndex(ForceInit)
	{}

	/** This version of the constructor supports Warp and Weft Scale */
	CHAOS_API FPBDAreaSpringConstraints(
		const FSolverParticlesRange & Particles,
		const FTriangleMesh& TriangleMesh,
		const TArray<TVec3<FVec2f>>&FaceVertexPatternPositions,
		const TMap<FString, TConstArrayView<FRealSingle>>&WeightMaps,
		const FCollectionPropertyConstFacade & PropertyCollection,
		bool bTrimKinematicConstraints = false);

	virtual ~FPBDAreaSpringConstraints() override = default;

	CHAOS_API void SetProperties(
		const FCollectionPropertyConstFacade& PropertyCollection,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps);

	CHAOS_API void ApplyProperties(const FSolverReal Dt, const int32 NumIterations);

	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(AreaSpringStiffness, float);
	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(AreaSpringWarpScale, float);
	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(AreaSpringWeftScale, float);

private:

	void InitFromPatternData(const TArray<TVec3<FVec2f>>& FaceVertexPatternPositions, const FTriangleMesh& TriangleMesh);
	void UpdateDists();

	using FPBDAxialSpringConstraints::Constraints;
	using FPBDAxialSpringConstraints::Barys;
	using FPBDAxialSpringConstraints::Stiffness;
	using FPBDAxialSpringConstraints::ParticleOffset;
	using FPBDAxialSpringConstraints::ParticleCount;

	bool bWarpWeftScaleEnabled = false;
	FPBDWeightMap WarpScale;
	FPBDWeightMap WeftScale;
	TArray<FSolverReal> BaseDists; // Without Warp/Weft Scale applied
	TArray<FSolverVec2> WarpWeftScaleBaseMultipliers;

	UE_CHAOS_DECLARE_INDEXED_PROPERTYCOLLECTION_NAME(AreaSpringStiffness, float);
	UE_CHAOS_DECLARE_INDEXED_PROPERTYCOLLECTION_NAME(AreaSpringWarpScale, float);
	UE_CHAOS_DECLARE_INDEXED_PROPERTYCOLLECTION_NAME(AreaSpringWeftScale, float);
};

}  // End namespace Chaos::Softs

#if !defined(CHAOS_AXIAL_SPRING_ISPC_ENABLED_DEFAULT)
#define CHAOS_AXIAL_SPRING_ISPC_ENABLED_DEFAULT 1
#endif

#if !defined(USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING)
#define USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING 0
#endif

// Support run-time toggling on supported platforms in non-shipping configurations
#if !INTEL_ISPC || (UE_BUILD_SHIPPING && !USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING)
static constexpr bool bChaos_AxialSpring_ISPC_Enabled = INTEL_ISPC && CHAOS_AXIAL_SPRING_ISPC_ENABLED_DEFAULT;
#else
extern CHAOS_API bool bChaos_AxialSpring_ISPC_Enabled;
#endif
