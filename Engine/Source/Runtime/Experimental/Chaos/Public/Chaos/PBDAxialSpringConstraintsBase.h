// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// HEADER_UNIT_SKIP - Internal

#include "Chaos/PBDSoftsEvolutionFwd.h"
#include "Chaos/PBDSoftsSolverParticles.h"
#include "Chaos/PBDStiffness.h"
#include "Chaos/SoftsSolverParticlesRange.h"
#include "Containers/Array.h"

namespace Chaos::Softs
{

class FPBDAxialSpringConstraintsBase
{
public:

	FPBDAxialSpringConstraintsBase(
		const FSolverParticlesRange& Particles,
		const TArray<TVec3<int32>>& InConstraints,
		const TConstArrayView<FRealSingle>& StiffnessMultipliers,
		const FSolverVec2& InStiffness,
		bool bTrimKinematicConstraints,
		FSolverReal MaxStiffness = FPBDStiffness::DefaultPBDMaxStiffness)
		: Constraints(TrimConstraints(InConstraints,
			[&Particles, bTrimKinematicConstraints](int32 Index0, int32 Index1, int32 Index2)
			{
				return bTrimKinematicConstraints && Particles.InvM(Index0) == (FSolverReal)0. && Particles.InvM(Index1) == (FSolverReal)0. && Particles.InvM(Index2) == (FSolverReal)0.;
			}))
		, ParticleOffset(0)
		, ParticleCount(Particles.GetRangeSize())
		, Stiffness(
			InStiffness,
			StiffnessMultipliers,
			TConstArrayView<TVec3<int32>>(Constraints),
			ParticleOffset,
			ParticleCount,
			FPBDStiffness::DefaultTableSize,
			FPBDStiffness::DefaultParameterFitBase,
			MaxStiffness)
	{
		Init(Particles, bTrimKinematicConstraints);
	}

	FPBDAxialSpringConstraintsBase(
		const FSolverParticles& Particles,
		int32 InParticleOffset,
		int32 InParticleCount,
		const TArray<TVec3<int32>>& InConstraints,
		const TConstArrayView<FRealSingle>& StiffnessMultipliers,
		const FSolverVec2& InStiffness,
		bool bTrimKinematicConstraints,
		FSolverReal MaxStiffness = FPBDStiffness::DefaultPBDMaxStiffness)
		: Constraints(TrimConstraints(InConstraints,
			[&Particles, bTrimKinematicConstraints](int32 Index0, int32 Index1, int32 Index2)
			{
				return bTrimKinematicConstraints && Particles.InvM(Index0) == (FSolverReal)0. && Particles.InvM(Index1) == (FSolverReal)0. && Particles.InvM(Index2) == (FSolverReal)0.;
			}))
		, ParticleOffset(InParticleOffset)
		, ParticleCount(InParticleCount)
		, Stiffness(
			InStiffness,
			StiffnessMultipliers,
			TConstArrayView<TVec3<int32>>(Constraints),
			ParticleOffset,
			ParticleCount,
			FPBDStiffness::DefaultTableSize,
			FPBDStiffness::DefaultParameterFitBase,
			MaxStiffness)
	{
		Init(Particles, bTrimKinematicConstraints);
	}

	virtual ~FPBDAxialSpringConstraintsBase() = default;

	void SetProperties(const FSolverVec2& InStiffness) { Stiffness.SetWeightedValue(InStiffness); }

	void ApplyProperties(const FSolverReal Dt, const int32 NumIterations) { Stiffness.ApplyPBDValues(Dt, NumIterations); }

	const TArray<TVec3<int32>>& GetConstraints() const { return Constraints; }
	const TArray<FSolverReal>& GetBarys() const { return Barys; }

	void ResetRestLengths(const TConstArrayView<FSolverVec3>& Positions)
	{
		CalculateRestLengths(Positions, TArrayView<FSolverReal>(Dists));
	}

protected:

	void CalculateRestLengths(const TConstArrayView<FSolverVec3>& Positions, const TArrayView<FSolverReal>& DistsToCompute) const
	{
		check(DistsToCompute.Num() == Constraints.Num());
		for (int32 Index = 0; Index < Constraints.Num(); ++Index)
		{

			const FSolverVec3& P1 = Positions[Constraints[Index][0]];
			const FSolverVec3& P2 = Positions[Constraints[Index][1]];
			const FSolverVec3& P3 = Positions[Constraints[Index][2]];
			const FSolverVec3 P = (P2 - P3) * Barys[Index] + P3;
			DistsToCompute[Index] = (P1 - P).Size();
		}
	}

	template<typename VectorType>
	inline VectorType GetSpringDiff(const VectorType& P1, const VectorType& P2, const VectorType& P3, const FSolverReal Bary) const
	{
		return P1 - ((P2 - P3) * Bary + P3);
	}

	template<typename SolverParticlesOrRange>
	inline FSolverVec3 GetDelta(const SolverParticlesOrRange& Particles, const int32 ConstraintIndex, const FSolverReal ExpStiffnessValue) const
	{
		const TVec3<int32>& Constraint = Constraints[ConstraintIndex];
		const int32 i1 = Constraint[0];
		const int32 i2 = Constraint[1];
		const int32 i3 = Constraint[2];
		const FSolverReal PInvMass = Particles.InvM(i3) * ((FSolverReal)1. - Barys[ConstraintIndex]) + Particles.InvM(i2) * Barys[ConstraintIndex];
		if (Particles.InvM(i1) == (FSolverReal)0. && PInvMass == (FSolverReal)0.)
		{
			return FSolverVec3((FSolverReal)0.);
		}
		const FSolverVec3& P1 = Particles.P(i1);
		const FSolverVec3& P2 = Particles.P(i2);
		const FSolverVec3& P3 = Particles.P(i3);
		const FSolverVec3 Difference = GetSpringDiff(P1, P2, P3, Barys[ConstraintIndex]);
		const FSolverReal Distance = Difference.Size();
		if (UNLIKELY(Distance <= (FSolverReal)UE_SMALL_NUMBER))
		{
			return FSolverVec3((FSolverReal)0.);
		}
		const FSolverVec3 Direction = Difference / Distance;
		const FSolverVec3 Delta = (Distance - Dists[ConstraintIndex]) * Direction;
		const FSolverReal CombinedInvMass = PInvMass + Particles.InvM(i1);
		checkSlow(CombinedInvMass > (FSolverReal)1e-7);
		return ExpStiffnessValue * Delta / CombinedInvMass;
	}

private:
	template<typename SolverParticlesOrRange>
	FSolverReal FindBary(const SolverParticlesOrRange& Particles, const int32 i1, const int32 i2, const int32 i3)
	{
		const FSolverVec3& P1 = Particles.GetX(i1);
		const FSolverVec3& P2 = Particles.GetX(i2);
		const FSolverVec3& P3 = Particles.GetX(i3);
		const FSolverVec3& P32 = P3 - P2;
		const FSolverReal Bary = FSolverVec3::DotProduct(P32, P3 - P1) / P32.SizeSquared();
		return FMath::Clamp(Bary, (FSolverReal)0., (FSolverReal)1.);
	}

	template<typename Predicate>
	TArray<TVec3<int32>> TrimConstraints(const TArray<TVec3<int32>>& InConstraints, Predicate TrimPredicate)
	{
		TSet<TVec3<int32>> TrimmedConstraints;
		TrimmedConstraints.Reserve(InConstraints.Num());

		for (const TVec3<int32>& Constraint : InConstraints)
		{
			const int32 Index0 = Constraint[0];
			const int32 Index1 = Constraint[1];
			const int32 Index2 = Constraint[2];

			if (!TrimPredicate(Index0, Index1, Index2))
			{
				TrimmedConstraints.Add(
					Index0 <= Index1 ?
						Index1 <= Index2 ? TVec3<int32>(Index0, Index1, Index2) :
						Index0 <= Index2 ? TVec3<int32>(Index0, Index2, Index1) :
										   TVec3<int32>(Index2, Index0, Index1) :
					// Index1 < Index0
						Index0 <= Index2 ? TVec3<int32>(Index1, Index0, Index2) :
						Index1 <= Index2 ? TVec3<int32>(Index1, Index2, Index0) :
										   TVec3<int32>(Index2, Index1, Index0));
			}
		}
		return TrimmedConstraints.Array();
	}

	template<typename SolverParticlesOrRange>
	void Init(const SolverParticlesOrRange& Particles, bool bTrimKinematicConstraints)
	{
		Barys.Reset(Constraints.Num());
		Dists.Reset(Constraints.Num());

		for (int32 Index = 0; Index < Constraints.Num();)
		{
			TVec3<int32>& Constraint = Constraints[Index];
			int32 i1 = Constraint[0];
			int32 i2 = Constraint[1];
			int32 i3 = Constraint[2];
			// Find Bary closest to 0.5
			const FSolverReal Bary1 = FindBary(Particles, i1, i2, i3);
			const FSolverReal Bary2 = FindBary(Particles, i2, i3, i1);
			const FSolverReal Bary3 = FindBary(Particles, i3, i1, i2);
			FSolverReal Bary = Bary1;
			const FSolverReal Bary1dist = FGenericPlatformMath::Abs(Bary1 - (FSolverReal)0.5);
			const FSolverReal Bary2dist = FGenericPlatformMath::Abs(Bary2 - (FSolverReal)0.5);
			const FSolverReal Bary3dist = FGenericPlatformMath::Abs(Bary3 - (FSolverReal)0.5);
			if (Bary3dist < Bary2dist && Bary3dist < Bary1dist)
			{
				Constraint[0] = i3;
				Constraint[1] = i1;
				Constraint[2] = i2;
				Bary = Bary3;
			}
			else if (Bary2dist < Bary1dist && Bary2dist < Bary3dist)
			{
				Constraint[0] = i2;
				Constraint[1] = i3;
				Constraint[2] = i1;
				Bary = Bary2;
			}
			// Reset as they may have changed
			i1 = Constraint[0];
			i2 = Constraint[1];
			i3 = Constraint[2];
			const FSolverVec3& P1 = Particles.GetX(i1);
			const FSolverVec3& P2 = Particles.GetX(i2);
			const FSolverVec3& P3 = Particles.GetX(i3);
			const FSolverVec3 P = (P2 - P3) * Bary + P3;

			if (bTrimKinematicConstraints)
			{
				const FSolverReal PInvMass = Particles.InvM(i3) * ((FSolverReal)1. - Bary) + Particles.InvM(i2) * Bary;
				const FSolverReal CombinedInvMass = PInvMass + Particles.InvM(i1);
				if (CombinedInvMass < UE_SMALL_NUMBER)
				{
					Constraints.RemoveAtSwap(Index);
					continue;
				}
			}

			Barys.Add(Bary);
			Dists.Add((P1 - P).Size());
			++Index;
		}
		check(Barys.Num() == Constraints.Num());
		check(Dists.Num() == Constraints.Num());
	}

protected:
	TArray<TVec3<int32>> Constraints;
	const int32 ParticleOffset;
	const int32 ParticleCount;
	TArray<FSolverReal> Barys;
	TArray<FSolverReal> Dists;
	FPBDStiffness Stiffness;
};

}  // End namespace Chaos::Softs
