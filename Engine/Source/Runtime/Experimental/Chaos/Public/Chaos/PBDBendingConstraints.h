// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/PBDBendingConstraintsBase.h"
#include "Chaos/CollectionPropertyFacade.h"

namespace Chaos::Softs
{

class FPBDBendingConstraints : public FPBDBendingConstraintsBase
{
	typedef FPBDBendingConstraintsBase Base;

public:
	static bool IsEnabled(const FCollectionPropertyConstFacade& PropertyCollection)
	{
		return IsBendingElementStiffnessEnabled(PropertyCollection, false);
	}	
	
	FPBDBendingConstraints(const FSolverParticlesRange& InParticles,
		TArray<TVec4<int32>>&& InConstraints,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
		const FCollectionPropertyConstFacade& PropertyCollection,
		bool bTrimKinematicConstraints = false)
		: Base(
			InParticles,
			MoveTemp(InConstraints),
			WeightMaps.FindRef(GetBendingElementStiffnessString(PropertyCollection, BendingElementStiffnessName.ToString())),
			WeightMaps.FindRef(GetBucklingRatioString(PropertyCollection, BucklingRatioName.ToString())),
			WeightMaps.FindRef(GetBucklingStiffnessString(PropertyCollection, BucklingStiffnessName.ToString())),
			GetRestAngleMapFromCollection(WeightMaps, PropertyCollection),
			FSolverVec2(GetWeightedFloatBendingElementStiffness(PropertyCollection, 1.f)),
			FSolverVec2(GetWeightedFloatBucklingRatio(PropertyCollection, 0.f)),  // BucklingRatio is clamped in base class
			FSolverVec2(GetWeightedFloatBucklingStiffness(PropertyCollection, 1.f)),
			GetRestAngleValueFromCollection(PropertyCollection),
			(ERestAngleConstructionType)GetRestAngleType(PropertyCollection, (int32)ERestAngleConstructionType::Use3DRestAngles),
			bTrimKinematicConstraints)
		, BendingElementStiffnessIndex(PropertyCollection)
		, BucklingRatioIndex(PropertyCollection)
		, BucklingStiffnessIndex(PropertyCollection)
		, FlatnessRatioIndex(PropertyCollection)
		, RestAngleIndex(PropertyCollection)
		, RestAngleTypeIndex(PropertyCollection)
	{
		InitColor(InParticles);
	}

	FPBDBendingConstraints(const FSolverParticles& InParticles,
		int32 InParticleOffset,
		int32 InParticleCount,
		TArray<TVec4<int32>>&& InConstraints,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
		const FCollectionPropertyConstFacade& PropertyCollection,
		bool bTrimKinematicConstraints = false)
		: Base(
			InParticles,
			InParticleOffset,
			InParticleCount,
			MoveTemp(InConstraints),
			WeightMaps.FindRef(GetBendingElementStiffnessString(PropertyCollection, BendingElementStiffnessName.ToString())),
			WeightMaps.FindRef(GetBucklingRatioString(PropertyCollection, BucklingRatioName.ToString())),
			WeightMaps.FindRef(GetBucklingStiffnessString(PropertyCollection, BucklingStiffnessName.ToString())),
			GetRestAngleMapFromCollection(WeightMaps, PropertyCollection),
			FSolverVec2(GetWeightedFloatBendingElementStiffness(PropertyCollection, 1.f)),
			FSolverVec2(GetWeightedFloatBucklingRatio(PropertyCollection, 0.f)),  // BucklingRatio is clamped in base class
			FSolverVec2(GetWeightedFloatBucklingStiffness(PropertyCollection, 1.f)),
			GetRestAngleValueFromCollection(PropertyCollection),
			(ERestAngleConstructionType)GetRestAngleType(PropertyCollection, (int32)ERestAngleConstructionType::Use3DRestAngles),
			bTrimKinematicConstraints)
		, BendingElementStiffnessIndex(PropertyCollection)
		, BucklingRatioIndex(PropertyCollection)
		, BucklingStiffnessIndex(PropertyCollection)
		, FlatnessRatioIndex(PropertyCollection)
		, RestAngleIndex(PropertyCollection)
		, RestAngleTypeIndex(PropertyCollection)
	{
		InitColor(InParticles);
	}

	FPBDBendingConstraints(const FSolverParticles& InParticles,
		int32 ParticleOffset,
		int32 ParticleCount,
		TArray<TVec4<int32>>&& InConstraints,
		const TConstArrayView<FRealSingle>& StiffnessMultipliers,
		const TConstArrayView<FRealSingle>& BucklingRatioMultipliers,
		const TConstArrayView<FRealSingle>& BucklingStiffnessMultipliers,
		const TConstArrayView<FRealSingle>& RestAngleMap,
		const FSolverVec2& InStiffness,
		const FSolverVec2& InBucklingRatio,
		const FSolverVec2& InBucklingStiffness,
		const FSolverVec2& RestAngleValue,
		ERestAngleConstructionType RestAngleConstructionType,
		bool bTrimKinematicConstraints = false)
		: Base(
			InParticles,
			ParticleOffset,
			ParticleCount,
			MoveTemp(InConstraints),
			StiffnessMultipliers,
			BucklingRatioMultipliers,
			BucklingStiffnessMultipliers,
			RestAngleMap,
			InStiffness,
			InBucklingRatio,
			InBucklingStiffness,
			RestAngleValue,
			RestAngleConstructionType,
			bTrimKinematicConstraints)
		, BendingElementStiffnessIndex(ForceInit)
		, BucklingRatioIndex(ForceInit)
		, BucklingStiffnessIndex(ForceInit)
		, FlatnessRatioIndex(ForceInit)
		, RestAngleIndex(ForceInit)
		, RestAngleTypeIndex(ForceInit)
	{
		InitColor(InParticles);
	}

	UE_DEPRECATED(5.5, "Use constructor with BucklingRatioMultipliers")
	FPBDBendingConstraints(const FSolverParticles& InParticles,
		int32 ParticleOffset,
		int32 ParticleCount,
		TArray<TVec4<int32>>&& InConstraints,
		const TConstArrayView<FRealSingle>& StiffnessMultipliers,
		const TConstArrayView<FRealSingle>& BucklingStiffnessMultipliers,
		const FSolverVec2& InStiffness,
		const FSolverReal InBucklingRatio,
		const FSolverVec2& InBucklingStiffness,
		bool bTrimKinematicConstraints = false)
		: FPBDBendingConstraints(
			InParticles, 
			ParticleOffset, 
			ParticleCount, 
			MoveTemp(InConstraints), 
			StiffnessMultipliers, 
			TConstArrayView<FRealSingle>(), 
			BucklingStiffnessMultipliers,
			TConstArrayView<FRealSingle>(),
			InStiffness, 
			FSolverVec2(InBucklingRatio), 
			InBucklingStiffness,
			FSolverVec2((FSolverReal)0.f),
			ERestAngleConstructionType::Use3DRestAngles,
			bTrimKinematicConstraints)
	{}

	virtual ~FPBDBendingConstraints() override = default;

	using Base::SetProperties;

	CHAOS_API void SetProperties(
		const FCollectionPropertyConstFacade& PropertyCollection,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps);

	template<typename SolverParticlesOrRange>
	CHAOS_API void Apply(SolverParticlesOrRange& InParticles, const FSolverReal Dt) const;

	const TArray<int32>& GetConstraintsPerColorStartIndex() const { return ConstraintsPerColorStartIndex; }

	void ResetRestLengths(const TConstArrayView<FSolverVec3>& Positions,
		const FCollectionPropertyConstFacade& PropertyCollection,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps)
	{
		CalculateRestAngles(Positions, ParticleOffset, ParticleCount,
			GetRestAngleMapFromCollection(WeightMaps, PropertyCollection),
			GetRestAngleValueFromCollection(PropertyCollection),
			(ERestAngleConstructionType)GetRestAngleType(PropertyCollection, (int32)ERestAngleConstructionType::Use3DRestAngles));
	}

	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(BendingElementStiffness, float);
	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(BucklingRatio, float);
	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(BucklingStiffness, float);
	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(FlatnessRatio, float);
	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(RestAngle, float);
	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(RestAngleType, int32);

private:
	template<typename SolverParticlesOrRange>
	CHAOS_API void InitColor(const SolverParticlesOrRange& InParticles);
	template<typename SolverParticlesOrRange>
	void ApplyHelper(SolverParticlesOrRange& InParticles, const FSolverReal Dt, const int32 ConstraintIndex, const FSolverReal ExpStiffnessValue, const FSolverReal ExpBucklingValue) const;

	TConstArrayView<FRealSingle> GetRestAngleMapFromCollection(
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
		const FCollectionPropertyConstFacade& PropertyCollection)
	{
		const ERestAngleConstructionType ConstructionType = (ERestAngleConstructionType)GetRestAngleType(PropertyCollection, (int32)ERestAngleConstructionType::Use3DRestAngles);

		switch (ConstructionType)
		{
		default:
		case ERestAngleConstructionType::Use3DRestAngles:
			return TConstArrayView<FRealSingle>(); // Unused
		case ERestAngleConstructionType::FlatnessRatio:
			return WeightMaps.FindRef(GetFlatnessRatioString(PropertyCollection, FlatnessRatioName.ToString()));
		case ERestAngleConstructionType::ExplicitRestAngles:
			return WeightMaps.FindRef(GetRestAngleString(PropertyCollection, RestAngleName.ToString()));
		}
	}

	FSolverVec2 GetRestAngleValueFromCollection(const FCollectionPropertyConstFacade& PropertyCollection)
	{
		const ERestAngleConstructionType ConstructionType = (ERestAngleConstructionType)GetRestAngleType(PropertyCollection, (int32)ERestAngleConstructionType::Use3DRestAngles);

		switch (ConstructionType)
		{
		default:
		case ERestAngleConstructionType::Use3DRestAngles:
			return FSolverVec2(0.f); // Unused
		case ERestAngleConstructionType::FlatnessRatio:
			return FSolverVec2(GetWeightedFloatFlatnessRatio(PropertyCollection, 0.f));
		case ERestAngleConstructionType::ExplicitRestAngles:
			return FSolverVec2(GetWeightedFloatRestAngle(PropertyCollection, 0.f));
		}
	}

	using Base::Constraints;
	using Base::ParticleOffset;
	using Base::ParticleCount;
	using Base::Stiffness;
	using Base::BucklingRatioWeighted;
	using Base::BucklingStiffness;

	TArray<int32> ConstraintsPerColorStartIndex; // Constraints are ordered so each batch is contiguous. This is ColorNum + 1 length so it can be used as start and end.

	UE_CHAOS_DECLARE_INDEXED_PROPERTYCOLLECTION_NAME(BendingElementStiffness, float);
	UE_CHAOS_DECLARE_INDEXED_PROPERTYCOLLECTION_NAME(BucklingRatio, float);
	UE_CHAOS_DECLARE_INDEXED_PROPERTYCOLLECTION_NAME(BucklingStiffness, float);
	UE_CHAOS_DECLARE_INDEXED_PROPERTYCOLLECTION_NAME(FlatnessRatio, float);
	UE_CHAOS_DECLARE_INDEXED_PROPERTYCOLLECTION_NAME(RestAngle, float);
	UE_CHAOS_DECLARE_INDEXED_PROPERTYCOLLECTION_NAME(RestAngleType, int32);
};

}  // End namespace Chaos::Softs
