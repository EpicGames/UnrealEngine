// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/PBDSoftsEvolutionFwd.h"
#include "Chaos/PBDSoftsSolverParticles.h"
#include "Chaos/SoftsSolverParticlesRange.h"
#include "Chaos/PBDStiffness.h"
#include "Chaos/ParticleRule.h"
#include "Containers/StaticArray.h"

namespace Chaos::Softs
{

class FPBDBendingConstraintsBase
{
public:
	enum class ERestAngleConstructionType : uint8
	{
		Use3DRestAngles,
		FlatnessRatio,
		ExplicitRestAngles
	};

	FPBDBendingConstraintsBase(const FSolverParticlesRange& InParticles,
		TArray<TVec4<int32>>&& InConstraints,
		const TConstArrayView<FRealSingle>& StiffnessMultipliers,
		const TConstArrayView<FRealSingle>& BucklingRatioMultipliers,
		const TConstArrayView<FRealSingle>& BucklingStiffnessMultipliers,
		const TConstArrayView<FRealSingle>& RestAngleMap,
		const FSolverVec2& InStiffness,
		const FSolverVec2& InBucklingRatio,
		const FSolverVec2& InBucklingStiffness,
		const FSolverVec2& RestAngleValue,
		ERestAngleConstructionType RestAngleConstructionType,
		bool bTrimKinematicConstraints = false,
		FSolverReal MaxStiffness = FPBDStiffness::DefaultPBDMaxStiffness)
		: Constraints(bTrimKinematicConstraints ? TrimKinematicConstraints(InConstraints, InParticles) : MoveTemp(InConstraints))
		, ConstraintSharedEdges(ExtractConstraintSharedEdges(Constraints))
		, ParticleOffset(0)
		, ParticleCount(InParticles.GetRangeSize())
		, Stiffness(
			InStiffness,
			StiffnessMultipliers,
			TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
			ParticleOffset,
			ParticleCount,
			FPBDStiffness::DefaultTableSize,
			FPBDStiffness::DefaultParameterFitBase,
			MaxStiffness)
		, BucklingRatioWeighted(
			InBucklingRatio.ClampAxes((FSolverReal)0., (FSolverReal)1.),
			BucklingRatioMultipliers,
			TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
			ParticleOffset,
			ParticleCount)
		, BucklingStiffness(
			InBucklingStiffness,
			BucklingStiffnessMultipliers,
			TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
			ParticleOffset,
			ParticleCount,
			FPBDStiffness::DefaultTableSize,
			FPBDStiffness::DefaultParameterFitBase,
			MaxStiffness)
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		, BucklingRatio((FSolverReal)BucklingRatioWeighted)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		CalculateRestAngles(InParticles.XArray(), ParticleOffset, ParticleCount, RestAngleMap, RestAngleValue, RestAngleConstructionType);
	}

	UE_DEPRECATED(5.5, "Use a constructor with BucklingRatioMultipliers.")
	FPBDBendingConstraintsBase(const FSolverParticlesRange& InParticles,
		TArray<TVec4<int32>>&& InConstraints,
		const TConstArrayView<FRealSingle>& StiffnessMultipliers,
		const TConstArrayView<FRealSingle>& BucklingStiffnessMultipliers,
		const TConstArrayView<FRealSingle>& RestAngleMap,
		const FSolverVec2& InStiffness,
		const FSolverReal InBucklingRatio,
		const FSolverVec2& InBucklingStiffness,
		const FSolverVec2& RestAngleValue,
		ERestAngleConstructionType RestAngleConstructionType,
		bool bTrimKinematicConstraints = false,
		FSolverReal MaxStiffness = FPBDStiffness::DefaultPBDMaxStiffness)
		: FPBDBendingConstraintsBase(
			InParticles,
			MoveTemp(InConstraints),
			StiffnessMultipliers,
			TConstArrayView<float>(),
			BucklingStiffnessMultipliers,
			RestAngleMap,
			InStiffness,
			FSolverVec2(InBucklingRatio),
			InBucklingStiffness,
			RestAngleValue,
			RestAngleConstructionType,
			bTrimKinematicConstraints,
			MaxStiffness)
	{}

	FPBDBendingConstraintsBase(const FSolverParticles& InParticles,
		int32 InParticleOffset,
		int32 InParticleCount,
		TArray<TVec4<int32>>&& InConstraints,
		const TConstArrayView<FRealSingle>& StiffnessMultipliers,
		const TConstArrayView<FRealSingle>& BucklingRatioMultipliers,
		const TConstArrayView<FRealSingle>& BucklingStiffnessMultipliers,
		const TConstArrayView<FRealSingle>& RestAngleMap,
		const FSolverVec2& InStiffness,
		const FSolverVec2& InBucklingRatio,
		const FSolverVec2& InBucklingStiffness,
		const FSolverVec2& RestAngleValue,
		ERestAngleConstructionType RestAngleConstructionType,
		bool bTrimKinematicConstraints = false,
		FSolverReal MaxStiffness = FPBDStiffness::DefaultPBDMaxStiffness)
		: Constraints(bTrimKinematicConstraints ? TrimKinematicConstraints(InConstraints, InParticles): MoveTemp(InConstraints))
		, ConstraintSharedEdges(ExtractConstraintSharedEdges(Constraints))
		, ParticleOffset(InParticleOffset)
		, ParticleCount(InParticleCount)
		, Stiffness(
			InStiffness,
			StiffnessMultipliers,
			TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
			ParticleOffset,
			ParticleCount,
			FPBDStiffness::DefaultTableSize,
			FPBDStiffness::DefaultParameterFitBase,
			MaxStiffness)
		, BucklingRatioWeighted(
			InBucklingRatio.ClampAxes((FSolverReal)0., (FSolverReal)1.),
			BucklingRatioMultipliers,
			TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
			ParticleOffset,
			ParticleCount)
		, BucklingStiffness(
			InBucklingStiffness,
			BucklingStiffnessMultipliers,
			TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
			ParticleOffset,
			ParticleCount,
			FPBDStiffness::DefaultTableSize,
			FPBDStiffness::DefaultParameterFitBase,
			MaxStiffness)
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		, BucklingRatio((FSolverReal)BucklingRatioWeighted)
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		CalculateRestAngles(InParticles.XArray(), ParticleOffset, ParticleCount, RestAngleMap, RestAngleValue, RestAngleConstructionType);
	}

	UE_DEPRECATED(5.5, "Use a constructor with BucklingRatioMultipliers.")
	FPBDBendingConstraintsBase(const FSolverParticles& InParticles,
		int32 InParticleOffset,
		int32 InParticleCount,
		TArray<TVec4<int32>>&& InConstraints,
		const TConstArrayView<FRealSingle>& StiffnessMultipliers,
		const TConstArrayView<FRealSingle>& BucklingStiffnessMultipliers,
		const TConstArrayView<FRealSingle>& RestAngleMap,
		const FSolverVec2& InStiffness,
		const FSolverReal InBucklingRatio,
		const FSolverVec2& InBucklingStiffness,
		const FSolverVec2& RestAngleValue,
		ERestAngleConstructionType RestAngleConstructionType,
		bool bTrimKinematicConstraints = false,
		FSolverReal MaxStiffness = FPBDStiffness::DefaultPBDMaxStiffness)
		: FPBDBendingConstraintsBase(
			InParticles,
			InParticleOffset,
			InParticleCount,
			MoveTemp(InConstraints),
			StiffnessMultipliers,
			TConstArrayView<FRealSingle>(),
			BucklingStiffnessMultipliers,
			RestAngleMap,
			InStiffness,
			FSolverVec2(InBucklingRatio),
			InBucklingStiffness,
			RestAngleValue,
			RestAngleConstructionType,
			bTrimKinematicConstraints,
			MaxStiffness)
	{
	}

	UE_DEPRECATED(5.5, "Use a constructor with BucklingRatioMultipliers.")
	FPBDBendingConstraintsBase(const FSolverParticles& InParticles,
		int32 InParticleOffset,
		int32 InParticleCount,
		TArray<TVec4<int32>>&& InConstraints,
		const TConstArrayView<FRealSingle>& StiffnessMultipliers,
		const TConstArrayView<FRealSingle>& BucklingStiffnessMultipliers,
		const FSolverVec2& InStiffness,
		const FSolverReal InBucklingRatio,
		const FSolverVec2& InBucklingStiffness,
		bool bTrimKinematicConstraints = false,
		FSolverReal MaxStiffness = FPBDStiffness::DefaultPBDMaxStiffness)
		:FPBDBendingConstraintsBase(InParticles, InParticleOffset, InParticleCount, MoveTemp(InConstraints),
			StiffnessMultipliers, TConstArrayView<FRealSingle>(), BucklingStiffnessMultipliers, TConstArrayView<FRealSingle>(),
			InStiffness, FSolverVec2(InBucklingRatio), InBucklingStiffness, FSolverVec2((FSolverReal)0.f),
			ERestAngleConstructionType::Use3DRestAngles, bTrimKinematicConstraints, MaxStiffness)
	{}

	virtual ~FPBDBendingConstraintsBase() = default;

	// Update stiffness values
	void SetProperties(const FSolverVec2& InStiffness, const FSolverVec2& InBucklingRatio, const FSolverVec2& InBucklingStiffness)
	{ 
		Stiffness.SetWeightedValue(InStiffness);
		BucklingRatioWeighted.SetWeightedValue(InBucklingRatio.ClampAxes((FSolverReal)0., (FSolverReal)1.));
		BucklingStiffness.SetWeightedValue(InBucklingStiffness);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		BucklingRatio = (FSolverReal)BucklingRatioWeighted;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	UE_DEPRECATED(5.5, "Use version with FSolverVec2 InBucklingRatio.")
	void SetProperties(const FSolverVec2& InStiffness, const FSolverReal InBucklingRatio, const FSolverVec2& InBucklingStiffness)
	{
		SetProperties(InStiffness, FSolverVec2(InBucklingRatio), InBucklingStiffness);
	}

	// Update stiffness table, as well as the simulation stiffness exponent
	void ApplyProperties(const FSolverReal Dt, const int32 NumIterations)
	{
		Stiffness.ApplyPBDValues(Dt, NumIterations);
		BucklingRatioWeighted.ApplyValues();
		BucklingStiffness.ApplyPBDValues(Dt, NumIterations);
	}

	template<typename SolverParticlesOrRange>
	TStaticArray<FSolverVec3, 4> GetGradients(const SolverParticlesOrRange& InParticles, const int32 i) const
	{
		const TVec4<int32>& Constraint = Constraints[i];
		const FSolverVec3& P1 = InParticles.P(Constraint[0]);
		const FSolverVec3& P2 = InParticles.P(Constraint[1]);
		const FSolverVec3& P3 = InParticles.P(Constraint[2]);
		const FSolverVec3& P4 = InParticles.P(Constraint[3]);

		return CalcGradients(P1, P2, P3, P4);
	}

	template<typename SolverParticlesOrRange>
	FSolverReal GetScalingFactor(const SolverParticlesOrRange& InParticles, const int32 i, const TStaticArray<FSolverVec3, 4>& Grads, const FSolverReal ExpStiffnessValue, const FSolverReal ExpBucklingValue) const
	{
		const TVec4<int32>& Constraint = Constraints[i];
		const int32 i1 = Constraint[0];
		const int32 i2 = Constraint[1];
		const int32 i3 = Constraint[2];
		const int32 i4 = Constraint[3];
		const FSolverVec3& P1 = InParticles.P(i1);
		const FSolverVec3& P2 = InParticles.P(i2);
		const FSolverVec3& P3 = InParticles.P(i3);
		const FSolverVec3& P4 = InParticles.P(i4);
		const FSolverReal Angle = CalcAngle(P1, P2, P3, P4);
		const FSolverReal Denom = (InParticles.InvM(i1) * Grads[0].SizeSquared() + InParticles.InvM(i2) * Grads[1].SizeSquared() + InParticles.InvM(i3) * Grads[2].SizeSquared() + InParticles.InvM(i4) * Grads[3].SizeSquared());

		const FSolverReal StiffnessValue = IsBuckled[i] ? ExpBucklingValue : ExpStiffnessValue;

		constexpr FSolverReal SingleStepAngleLimit = (FSolverReal)(UE_PI * .25f); // this constraint is very non-linear. taking large steps is not accurate
		const FSolverReal Delta = FMath::Clamp(StiffnessValue * (Angle - RestAngles[i]), -SingleStepAngleLimit, SingleStepAngleLimit);
		return SafeDivide(Delta, Denom);
	}

	static FSolverReal CalcAngle(const FSolverVec3& P1, const FSolverVec3& P2, const FSolverVec3& P3, const FSolverVec3& P4)
	{
		const FSolverVec3 Normal1 = FSolverVec3::CrossProduct(P1 - P3, P2 - P3).GetSafeNormal();
		const FSolverVec3 Normal2 = FSolverVec3::CrossProduct(P2 - P4, P1 - P4).GetSafeNormal();

		const FSolverVec3 SharedEdge = (P2 - P1).GetSafeNormal();

		const FSolverReal CosPhi = FMath::Clamp(FSolverVec3::DotProduct(Normal1, Normal2), (FSolverReal)-1, (FSolverReal)1);
		const FSolverReal SinPhi = FMath::Clamp(FSolverVec3::DotProduct(FSolverVec3::CrossProduct(Normal2, Normal1), SharedEdge), (FSolverReal)-1, (FSolverReal)1);
		return FMath::Atan2(SinPhi, CosPhi);
	}

	UE_DEPRECATED(5.5, "This version is deprecated and will produce the incorrect result if BucklingRatio is weighted. Use version that takes a ConstraintIndex instead.")
	bool AngleIsBuckled(const FSolverReal Angle, const FSolverReal RestAngle) const
	{
		// Angle is 0 when completely flat. This is easier to think of in terms of Angle' = (PI - |Angle|), which is 0 when completely folded.
		// Consider buckled when Angle' <= BucklingRatio * RestAngle', and use buckling stiffness instead of stiffness.
		return UE_PI - FMath::Abs(Angle) < (FSolverReal)BucklingRatioWeighted * (UE_PI - FMath::Abs(RestAngle));
	}

	bool AngleIsBuckled(const FSolverReal Angle, const int32 ConstraintIndex) const
	{
		// Angle is 0 when completely flat. This is easier to think of in terms of Angle' = (PI - |Angle|), which is 0 when completely folded.
		// Consider buckled when Angle' <= BucklingRatio * RestAngle', and use buckling stiffness instead of stiffness.
		return UE_PI - FMath::Abs(Angle) < BucklingRatioWeighted.GetValue(ConstraintIndex) * (UE_PI - FMath::Abs(RestAngles[ConstraintIndex]));
	}

	template<typename SolverParticlesOrRange>
	CHAOS_API void Init(const SolverParticlesOrRange& InParticles);

	const TArray<FSolverReal>& GetRestAngles() const { return RestAngles; }
	const TArray<TVec4<int32>>& GetConstraints() const { return Constraints; }
	const TArray<bool>& GetIsBuckled() const { return IsBuckled; }

protected:
	template<class TNum>
	static TNum SafeDivide(const TNum& Numerator, const FSolverReal& Denominator)
	{
		if (Denominator > SMALL_NUMBER)
			return Numerator / Denominator;
		return TNum(0);
	}
	
	static TStaticArray<FSolverVec3, 4> CalcGradients(const FSolverVec3& P1, const FSolverVec3& P2, const FSolverVec3& P3, const FSolverVec3& P4, FSolverReal* OutAngle = nullptr)
	{
		TStaticArray<FSolverVec3, 4> Grads;
		// Calculated using Phi = atan2(SinPhi, CosPhi)
		// where SinPhi = (Normal1 ^ Normal2)*SharedEdgeNormalized, CosPhi = Normal1 * Normal2
		// Full gradients are calculated here, i.e., no simplifying assumptions around things like edge lengths being constant.
		const FSolverVec3 SharedEdgeNormalized = (P2 - P1).GetSafeNormal();
		const FSolverVec3 P13CrossP23 = FSolverVec3::CrossProduct(P1 - P3, P2 - P3);
		const FSolverReal Normal1Len = P13CrossP23.Size();
		const FSolverVec3 Normal1 = SafeDivide(P13CrossP23, Normal1Len);
		const FSolverVec3 P24CrossP14 = FSolverVec3::CrossProduct(P2 - P4, P1 - P4);
		const FSolverReal Normal2Len = P24CrossP14.Size();
		const FSolverVec3 Normal2 = SafeDivide(P24CrossP14, Normal2Len);

		const FSolverVec3 N2CrossN1 = FSolverVec3::CrossProduct(Normal2, Normal1);

		const FSolverReal CosPhi = FMath::Clamp(FSolverVec3::DotProduct(Normal1, Normal2), (FSolverReal)-1, (FSolverReal)1);
		const FSolverReal SinPhi = FMath::Clamp(FSolverVec3::DotProduct(N2CrossN1, SharedEdgeNormalized), (FSolverReal)-1, (FSolverReal)1);

		const FSolverVec3 DPhiDN1_OverNormal1Len = SafeDivide(CosPhi * FSolverVec3::CrossProduct(SharedEdgeNormalized, Normal2) - SinPhi * Normal2, Normal1Len);
		const FSolverVec3 DPhiDN2_OverNormal2Len = SafeDivide(CosPhi * FSolverVec3::CrossProduct(Normal1, SharedEdgeNormalized) - SinPhi * Normal1, Normal2Len);

		const FSolverVec3 DPhiDP13 = FSolverVec3::CrossProduct(P2 - P3, DPhiDN1_OverNormal1Len);
		const FSolverVec3 DPhiDP23 = FSolverVec3::CrossProduct(DPhiDN1_OverNormal1Len, P1 - P3);
		const FSolverVec3 DPhiDP24 = FSolverVec3::CrossProduct(P1 - P4, DPhiDN2_OverNormal2Len);
		const FSolverVec3 DPhiDP14 = FSolverVec3::CrossProduct(DPhiDN2_OverNormal2Len, P2 - P4);

		Grads[0] = DPhiDP13 + DPhiDP14;
		Grads[1] = DPhiDP23 + DPhiDP24;
		Grads[2] = -DPhiDP13 - DPhiDP23;
		Grads[3] = -DPhiDP14 - DPhiDP24;

		if (OutAngle)
		{
			*OutAngle = FMath::Atan2(SinPhi, CosPhi);
		}

		return Grads;
	}

	CHAOS_API void CalculateRestAngles(const TConstArrayView<FSolverVec3>& InPositions,
		int32 InParticleOffset,
		int32 InParticleCount,
		const TConstArrayView<FRealSingle>& RestAngleMap,
		const FSolverVec2& RestAngleValue,
		ERestAngleConstructionType RestAngleConstructionType);

private:
	template<typename SolverParticlesOrRange>
	static TArray<TVec4<int32>> TrimKinematicConstraints(const TArray<TVec4<int32>>& InConstraints, const SolverParticlesOrRange& InParticles)
	{
		TArray<TVec4<int32>> TrimmedConstraints;
		TrimmedConstraints.Reserve(InConstraints.Num());
		for (const TVec4<int32>& Constraint : InConstraints)
		{
			if (InParticles.InvM(Constraint[0]) != (FSolverReal)0. || InParticles.InvM(Constraint[1]) != (FSolverReal)0. || InParticles.InvM(Constraint[2]) != (FSolverReal)0. || InParticles.InvM(Constraint[3]) != (FSolverReal)0.)
			{
				TrimmedConstraints.Add(Constraint);
			}
		}
		TrimmedConstraints.Shrink();
		return TrimmedConstraints;
	}

	static TArray<TVec2<int32>> ExtractConstraintSharedEdges(const TArray<TVec4<int32>>& Constraints)
	{
		TArray<TVec2<int32>> ExtractedEdges;
		ExtractedEdges.Reserve(Constraints.Num());
		for (const TVec4<int32>& Constraint : Constraints)
		{
			ExtractedEdges.Emplace(Constraint[0], Constraint[1]);
		}
		return ExtractedEdges;
	}
	
protected:
	TArray<TVec4<int32>> Constraints;
	TArray<TVec2<int32>> ConstraintSharedEdges; // Only shared edges are used for calculating weighted stiffnesses.

	const int32 ParticleOffset;
	const int32 ParticleCount;

	FPBDStiffness Stiffness;
	FPBDWeightMap BucklingRatioWeighted;
	FPBDStiffness BucklingStiffness;

	TArray<FSolverReal> RestAngles;
	TArray<bool> IsBuckled;

	UE_DEPRECATED(5.5, "Scalar Buckling Ratio has been deprecated. Use BucklingRatioWeighted instead.")
	FSolverReal BucklingRatio;
};

}  // End namespace Chaos::Softs

#if !defined(CHAOS_BENDING_ISPC_ENABLED_DEFAULT)
#define CHAOS_BENDING_ISPC_ENABLED_DEFAULT 1
#endif

#if !defined(USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING)
#define USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING 0
#endif

// Support run-time toggling on supported platforms in non-shipping configurations
#if !INTEL_ISPC || (UE_BUILD_SHIPPING && !USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING)
static constexpr bool bChaos_Bending_ISPC_Enabled = INTEL_ISPC && CHAOS_BENDING_ISPC_ENABLED_DEFAULT;
#else
extern CHAOS_API bool bChaos_Bending_ISPC_Enabled;
#endif
