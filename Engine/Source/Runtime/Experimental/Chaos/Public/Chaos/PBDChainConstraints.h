// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// HEADER_UNIT_SKIP - This file does not compile anymore (Base ctor needs to be called)

#include "Chaos/Evolution/IndexedConstraintContainer.h"
#include "Chaos/PBDParticles.h"
#include "Chaos/Framework/Parallel.h"
#include "Chaos/PBDConstraintContainer.h"
#include "Chaos/DynamicParticles.h"
#include "Chaos/PBDParticles.h"

namespace Chaos
{
class FPBDChainConstraints : public FPBDIndexedConstraintContainer
{
public:
	FPBDChainConstraints(const FDynamicParticles& InParticles, TArray<TArray<int32>>&& Constraints, const FReal Coefficient = (FReal)1.)
	    : MConstraints(Constraints), MCoefficient(Coefficient)
	{
		MDists.SetNum(MConstraints.Num());
		PhysicsParallelFor(MConstraints.Num(), [&](int32 Index) {
			TArray<FReal> singledists;
			for (int i = 1; i < Constraints[Index].Num(); ++i)
			{
				const FVec3& P1 = InParticles.X(Constraints[Index][i - 1]);
				const FVec3& P2 = InParticles.X(Constraints[Index][i]);
				FReal Distance = (P1 - P2).Size();
				singledists.Add(Distance);
			}
			MDists[Index] = singledists;
		});
	}
	virtual ~FPBDChainConstraints() = default;

	void Apply(FPBDParticles& InParticles, const FReal Dt, const int32 InConstraintIndex) const
	{
		const int32 Index = InConstraintIndex;
		for (int i = 1; i < MConstraints[Index].Num(); ++i)
		{
			int32 P = MConstraints[Index][i];
			int32 PM1 = MConstraints[Index][i - 1];
			const FVec3& P1 = InParticles.P(PM1);
			const FVec3& P2 = InParticles.P(P);
			FVec3 Difference = P1 - P2;
			FReal Distance = Difference.Size();
			FVec3 Direction = Difference / Distance;
			FVec3 Delta = (Distance - MDists[Index][i - 1]) * Direction;
			if (i == 1)
			{
				InParticles.P(P) += Delta;
			}
			else
			{
				InParticles.P(P) += MCoefficient * Delta;
				InParticles.P(PM1) -= (1 - MCoefficient) * Delta;
			}
		}
	}

	void Apply(FPBDParticles& InParticles, const FReal Dt) const
	{
		// @todo(ccaulfield): Can we guarantee that no two chains are connected? Should we be checking that somewhere?
		PhysicsParallelFor(MConstraints.Num(), [&](int32 ConstraintIndex) {
			Apply(InParticles, Dt, ConstraintIndex);
		});
	}

	void Apply(FPBDParticles& InParticles, const FReal Dt, const TArray<int32>& InConstraintIndices) const
	{
		// @todo(ccaulfield): Can we guarantee that no two chains are connected? Should we be checking that somewhere?
		PhysicsParallelFor(InConstraintIndices.Num(), [&](int32 ConstraintIndicesIndex) {
			Apply(InParticles, Dt, InConstraintIndices[ConstraintIndicesIndex]);
		});
	}

private:
	using Base::GetConstraintIndex;
	using Base::SetConstraintIndex;
	  
	TArray<TArray<int32>> MConstraints;
	TArray<TArray<FReal>> MDists;
	FReal MCoefficient;
};

template<class T, int d>
class TPBDChainConstraints UE_DEPRECATED(4.27, "Deprecated. this class is to be deleted, use FPBDChainConstraints instead") = FPBDChainConstraints;

}
