// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/CollisionResolutionTypes.h"
#include "Chaos/Collision/CollisionApplyType.h"
#include "Chaos/Collision/CollisionConstraintAllocator.h"
#include "Chaos/Collision/CollisionContext.h"
#include "Chaos/Collision/PBDCollisionConstraint.h"
#include "Chaos/Collision/PBDCollisionConstraintHandle.h"
#include "Chaos/Collision/PBDCollisionSolverSettings.h"
#include "Chaos/PBDConstraintContainer.h"
#include "Framework/BufferedData.h"

#include <memory>
#include <queue>
#include <sstream>
#include "BoundingVolume.h"
#include "AABBTree.h"

namespace Chaos
{
class FImplicitObject;
class FPBDCollisionConstraints;
class FPBDRigidsSOAs;
class FPBDCollisionConstraint;

using FRigidBodyContactConstraintsPostComputeCallback = TFunction<void()>;
using FRigidBodyContactConstraintsPostApplyCallback = TFunction<void(const FReal Dt, const TArray<FPBDCollisionConstraintHandle*>&)>;
using FRigidBodyContactConstraintsPostApplyPushOutCallback = TFunction<void(const FReal Dt, const TArray<FPBDCollisionConstraintHandle*>&, bool)>;


namespace Private
{
	// The type of solver to use for collisions
	enum class ECollisionSolverType
	{
		GaussSeidel,
		GaussSeidelSimd,
		PartialJacobi,
	};
}

/**
 * A container and solver for collision constraints.
 * 
 * @todo(chaos): remove handles array
 */
class FPBDCollisionConstraints : public FPBDConstraintContainer
{
public:
	friend class FPBDCollisionConstraintHandle;

	using Base = FPBDIndexedConstraintContainer;

	// Collision constraints have intrusive pointers. An array of constraint pointers can be uased as an array of handle pointers
	using FHandles = TArrayView<FPBDCollisionConstraint* const>;
	using FConstHandles = TArrayView<const FPBDCollisionConstraint* const>;

	// For use by dependent types
	using FConstraintContainerHandle = FPBDCollisionConstraintHandle;		// Used by constraint rules

	CHAOS_API FPBDCollisionConstraints(const FPBDRigidsSOAs& InParticles, 
		TArrayCollectionArray<bool>& Collided, 
		const TArrayCollectionArray<TSerializablePtr<FChaosPhysicsMaterial>>& PhysicsMaterials, 
		const TArrayCollectionArray<TUniquePtr<FChaosPhysicsMaterial>>& PerParticlePhysicsMaterials,
		const THandleArray<FChaosPhysicsMaterial>* const SimMaterials,
		const int32 NumCollisionsPerBlock = 1000,
		const FReal RestitutionThreshold = FReal(2000));

	CHAOS_API virtual ~FPBDCollisionConstraints();

	const FString& GetName() const { return DetectorSettings.GetName(); }
	void SetName(const FSharedDebugName& InName) { DetectorSettings.EvolutionName = InName; }

	/**
	 * Whether this container provides constraint handles (simple solvers do not need them)
	 */
	bool GetHandlesEnabled() const { return bHandlesEnabled; }

	/**
	 * Put the container in "no handles" mode for use with simple solver. Must be called when empty of constraints (ideally right after creation).
	 */
	CHAOS_API void DisableHandles();

	/**
	 * @brief Enable or disable determinism.
	 * Support for determinism requires that we sort active constraints each tick, so there is additional cost.
	*/
	void SetIsDeterministic(const bool bInIsDeterministic);

	/**
	 * @brief Get determinism.
	 * Support for determinism requires that we sort active constraints each tick, so there is additional cost.
	*/
	bool GetIsDeterministic() const;

	/**
	 *  Clears the list of active constraints.
	 * @todo(chaos): This is only required because of the way events work (see AdvanceOneTimeStepTask::DoWork)
	*/
	CHAOS_API void BeginFrame();

	/**
	*  Destroy all constraints 
	*/
	CHAOS_API void Reset();


	/**
	 * @brief Called before collision detection to reset contacts
	*/
	CHAOS_API void BeginDetectCollisions();

	/**
	 * @brief Called after collision detection to finalize the contacts
	*/
	CHAOS_API void EndDetectCollisions();

	/**
	 * @brief Called after collision resolution in order to detect probes
	 */
	CHAOS_API void DetectProbeCollisions(FReal Dt);

	/**
	 * Apply modifiers to particle pair midphases
	 */
	UE_DEPRECATED(5.6, "Not supported. Use FPBDRigidsEvolutionGBF::ApplyMidPhaseModifier")
	void ApplyMidPhaseModifier(const TArray<ISimCallbackObject*>& MidPhaseModifiers, FReal Dt) {}

	/**
	 * Apply modifiers to CCD results
	 */
	CHAOS_API void ApplyCCDModifier(const TArray<ISimCallbackObject*>& CCDModifiers, FReal Dt);


	/**
	 * Apply modifiers to the constraints and specify which constraints should be disabled.
	 * You would probably call this in the PostComputeCallback. Prefer this to calling RemoveConstraints in a loop,
	 * so you don't have to worry about constraint iterator/indices changing.
	 */
	CHAOS_API void ApplyCollisionModifier(const TArray<ISimCallbackObject*>& CollisionModifiers, FReal Dt);


	/**
	* Remove the constraints associated with the ParticleHandle.
	*/
	CHAOS_API void RemoveConstraints(const TSet<FGeometryParticleHandle*>&  ParticleHandle);

	/**
	 * Does nothing for transient constraints such as collision constraints.
	 * Use RemoveConstraints for removing collision constraints.
	*/
	CHAOS_API virtual void DisconnectConstraints(const TSet<FGeometryParticleHandle*>& ParticleHandles) override;

	/**
	* Disable the constraints associated with the ParticleHandle.
	*/
	void DisableConstraints(const TSet<FGeometryParticleHandle*>& ParticleHandle) {}

	//
	// FConstraintContainer Implementation
	//
	virtual int32 GetNumConstraints() const override final { return NumConstraints(); }
	virtual void ResetConstraints() override final { Reset(); }
	CHAOS_API virtual void AddConstraintsToGraph(Private::FPBDIslandManager& IslandManager) override final;
	virtual void PrepareTick() override final {}
	virtual void UnprepareTick() override final {}

	CHAOS_API virtual TUniquePtr<FConstraintContainerSolver> CreateSceneSolver(const int32 Priority) override final;

	CHAOS_API virtual TUniquePtr<FConstraintContainerSolver> CreateGroupSolver(const int32 Priority) override final;

	// The type of solver we are creating
	Private::ECollisionSolverType GetSolverType() const
	{
		return CollisionSolverType;
	}

	// Set the solver type. NOTE: Any previously created solvers will not be recreated at this level. (See FPBDRigidsEvolutionGBF::UpdateCollisionSolverType)
	void SetSolverType(const Private::ECollisionSolverType InSolverType)
	{
		CollisionSolverType = InSolverType;
	}

	//
	// Member Access
	//

	void SetCanDisableContacts(bool bInCanDisableContacts)
	{
		bCanDisableContacts = bInCanDisableContacts;
	}

	bool GetCanDisableContacts() const
	{
		return bCanDisableContacts;
	}

	void SetRestitutionThreshold(FReal InRestitutionThreshold)
	{
		RestitutionThreshold = InRestitutionThreshold;
	}

	FReal GetRestitutionThreshold() const
	{
		return RestitutionThreshold;
	}

	void SetCollisionsEnabled(bool bInEnableCollisions)
	{
		bEnableCollisions = bInEnableCollisions;
	}

	bool GetCollisionsEnabled() const
	{
		return bEnableCollisions;
	}

	void SetRestitutionEnabled(bool bInEnableRestitution)
	{
		bEnableRestitution = bInEnableRestitution;
	}

	bool GetRestitutionEnabled() const
	{
		return bEnableRestitution;
	}

	void SetGravity(const FVec3& InGravity)
	{
		GravityDirection = InGravity;
		GravitySize = GravityDirection.SafeNormalize();
	}

	FVec3 GetGravityDirection() const
	{
		return GravityDirection;
	}

	FReal GetGravitySize() const
	{
		return GravitySize;
	}

	void SetMaxPushOutVelocity(const FReal InMaxPushOutVelocity)
	{
		SolverSettings.MaxPushOutVelocity = FMath::Max(InMaxPushOutVelocity, FReal(0));
	}

	void SetDepenetrationVelocity(const FRealSingle InVel)
	{
		// The user can specify a very large number up to float_max or any negative number to mean "infinity".
		// However, we don't use float_max for infinity because we want to be able to perform simple math on
		// it without numeric limit issues (search MaxDepenetrationVelocity).
		constexpr float MaxDepenetrationVelocity = 1e10f;	// [cm/s] almost speed of light :)

		if (InVel >= 0.0f)
		{
			SolverSettings.DepenetrationVelocity = FMath::Min(InVel, MaxDepenetrationVelocity);
		}
		else
		{
			SolverSettings.DepenetrationVelocity = MaxDepenetrationVelocity;
		}
	}

	void SetPositionFrictionIterations(const int32 InNumIterations)
	{
		SolverSettings.NumPositionFrictionIterations = InNumIterations;
	}

	void SetVelocityFrictionIterations(const int32 InNumIterations)
	{
		SolverSettings.NumVelocityFrictionIterations = InNumIterations;
	}

	void SetPositionShockPropagationIterations(const int32 InNumIterations)
	{
		SolverSettings.NumPositionShockPropagationIterations = InNumIterations;
	}

	void SetVelocityShockPropagationIterations(const int32 InNumIterations)
	{
		SolverSettings.NumVelocityShockPropagationIterations = InNumIterations;
	}

	UE_INTERNAL void SetUseSplitImpulse(bool bInUseSplitImpulse)
	{
		SolverSettings.bUseSplitImpulse = bInUseSplitImpulse;
	}

	bool IsShockPropagationEnabled() const
	{
		return (SolverSettings.NumPositionShockPropagationIterations > 0) || (SolverSettings.NumVelocityShockPropagationIterations > 0);
	}

	int32 NumConstraints() const
	{
		return GetConstraints().Num();
	}

	TArrayView<FPBDCollisionConstraint* const> GetConstraints() const
	{
		return ConstraintAllocator.GetConstraints();
	}

	CHAOS_API FHandles GetConstraintHandles() const;
	CHAOS_API FConstHandles GetConstConstraintHandles() const;

	CHAOS_API const FPBDCollisionConstraint& GetConstraint(int32 Index) const;
	CHAOS_API const FPBDCollisionConstraint& GetConstConstraint(int32 Index) const;

	Private::FCollisionConstraintAllocator& GetConstraintAllocator() { return ConstraintAllocator; }

	CHAOS_API void UpdateConstraintMaterialProperties(FPBDCollisionConstraint& Contact);

	const FPBDCollisionSolverSettings& GetSolverSettings() const { return SolverSettings; }

	void SetSolverSettings(const FPBDCollisionSolverSettings& InSettings)
	{
		SolverSettings = InSettings;
	}

	const FCollisionDetectorSettings& GetDetectorSettings() const { return DetectorSettings; }

	void SetDetectorSettings(const FCollisionDetectorSettings& InSettings)
	{
		DetectorSettings = InSettings;
	}

	void SetCullDistance(const FReal InCullDistance)
	{
		DetectorSettings.BoundsExpansion = InCullDistance;
	}

	void SetVelocityBoundsExpansion(const FReal BoundsVelocityMultiplier, const FReal MaxVelocityBoundsExpansion)
	{
		DetectorSettings.BoundsVelocityInflation = BoundsVelocityMultiplier;
		DetectorSettings.MaxVelocityBoundsExpansion = MaxVelocityBoundsExpansion;
	}

	void SetVelocityBoundsExpansionMACD(const FReal BoundsVelocityMultiplier, const FReal MaxVelocityBoundsExpansion)
	{
		DetectorSettings.BoundsVelocityInflationMACD = BoundsVelocityMultiplier;
		DetectorSettings.MaxVelocityBoundsExpansionMACD = MaxVelocityBoundsExpansion;
	}

	void SetAllowManifolds(const bool bAllowManifolds)
	{
		DetectorSettings.bAllowManifolds = bAllowManifolds;
	}

protected:
	CHAOS_API FPBDCollisionConstraint& GetConstraint(int32 Index);

	// Call PruneParticleEdgeCollisions on all particles with ECollisionConstraintFlags::CCF_SmoothEdgeCollisions set in CollisionFlags
	CHAOS_API void PruneEdgeCollisions();

private:
	const FPBDRigidsSOAs& Particles;

	Private::FCollisionConstraintAllocator ConstraintAllocator;
	int32 NumActivePointConstraints;
	TArray<FPBDCollisionConstraintHandle*> TempCollisions;	// Reused from tick to tick to build contact lists

	TArrayCollectionArray<bool>& MCollided;
	const TArrayCollectionArray<TSerializablePtr<FChaosPhysicsMaterial>>& MPhysicsMaterials;
	const TArrayCollectionArray<TUniquePtr<FChaosPhysicsMaterial>>& MPerParticlePhysicsMaterials;
	const THandleArray<FChaosPhysicsMaterial>* const SimMaterials;

	FReal RestitutionThreshold;
	bool bEnableCollisions;
	bool bEnableRestitution;
	bool bHandlesEnabled;
	bool bEnableEdgePruning;
	bool bIsDeterministic;

	// This is passed to IterationParameters. If true, then an iteration can cull a contact
	// permanently (ie, for the remaining iterations) if it is ignored due to culldistance.
	// This improves performance, but can decrease stability if contacts are culled prematurely.
	bool bCanDisableContacts;

	Private::ECollisionSolverType CollisionSolverType;

	// Used to determine constraint directions
	FVec3 GravityDirection;
	FReal GravitySize;

	// Settings for the low-level collision solvers
	FPBDCollisionSolverSettings SolverSettings;

	// Settings for collision detection
	FCollisionDetectorSettings DetectorSettings;
};

//
//
// Inlined FPBDCollisionConstraintHandle functions. Here to avoid circular deps
//
//

inline const FPBDCollisionConstraints* FPBDCollisionConstraintHandle::ConcreteContainer() const
{
	return static_cast<FPBDCollisionConstraints*>(ConstraintContainer);
}

inline FPBDCollisionConstraints* FPBDCollisionConstraintHandle::ConcreteContainer()
{
	return static_cast<FPBDCollisionConstraints*>(ConstraintContainer);
}

inline const FPBDCollisionConstraint& FPBDCollisionConstraintHandle::GetContact() const
{
	return *GetConstraint();
}

inline FPBDCollisionConstraint& FPBDCollisionConstraintHandle::GetContact()
{
	return *GetConstraint();
}

inline bool FPBDCollisionConstraintHandle::GetCCDEnabled() const
{
	return GetContact().GetCCDEnabled();
}

inline void FPBDCollisionConstraintHandle::SetEnabled(bool InEnabled)
{
	GetContact().SetDisabled(!InEnabled);
}

inline bool FPBDCollisionConstraintHandle::IsEnabled() const
{
	return !GetContact().GetDisabled();
}

inline bool FPBDCollisionConstraintHandle::IsProbe() const
{
	return GetContact().GetIsProbe();
}

inline FVec3 FPBDCollisionConstraintHandle::GetAccumulatedImpulse() const
{
	return GetContact().AccumulatedImpulse;
}

inline FParticlePair FPBDCollisionConstraintHandle::GetConstrainedParticles() const
{
	return { GetContact().GetParticle0(), GetContact().GetParticle1() };
}

}
