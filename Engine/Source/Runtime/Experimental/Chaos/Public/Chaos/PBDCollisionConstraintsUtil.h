// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/ExternalCollisionData.h"
#include "Chaos/PBDCollisionConstraints.h"
#include "Chaos/CollisionResolutionTypes.h"

namespace Chaos
{
	void CHAOS_API ComputeHashTable(const TArray<Chaos::FPBDCollisionConstraint>& ConstraintsArray,
									const FBox& BoundingBox, TMultiMap<int32, int32>& HashTableMap, const Chaos::FReal SpatialHashRadius);

	void CHAOS_API ComputeHashTable(const TArray<FCollidingData>& CollisionsArray,
									const FBox& BoundingBox, TMultiMap<int32, int32>& HashTableMap, const Chaos::FReal SpatialHashRadius);

	void CHAOS_API ComputeHashTable(const TArray<FCollidingDataExt>& CollisionsArray,
									const FBox& BoundingBox, TMultiMap<int32, int32>& HashTableMap, const Chaos::FReal SpatialHashRadius);

	void CHAOS_API ComputeHashTable(const TArray<FVector>& ParticleArray, const FBox& BoundingBox, 
									TMultiMap<int32, int32>& HashTableMap, const float SpatialHashRadius);

	void CHAOS_API ComputeHashTable(const TArray<FBreakingData>& BreakingsArray,
									const FBox& BoundingBox, TMultiMap<int32, int32>& HashTableMap, const Chaos::FReal SpatialHashRadius);

	void CHAOS_API ComputeHashTable(const TArray<FBreakingDataExt>& BreakingsArray,
									const FBox& BoundingBox, TMultiMap<int32, int32>& HashTableMap, const Chaos::FReal SpatialHashRadius);
}
