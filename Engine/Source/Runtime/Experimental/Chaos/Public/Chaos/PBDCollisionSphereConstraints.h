// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if !COMPILE_WITHOUT_UNREAL_SUPPORT
#include "Chaos/Array.h"
#include "Chaos/BoundingVolumeHierarchy.h"
#include "Chaos/Framework/Parallel.h"
#include "Chaos/Map.h"
#include "Chaos/PBDParticles.h"
#include "Chaos/PerParticleRule.h"
#include "Chaos/Sphere.h"

#include <algorithm>

// This is an approximation but only collides with spheres in the velocity direction which can hurt compared to all directions when it comes to thickness
namespace Chaos
{
class FPBDCollisionSphereConstraints : public FPerParticleRule
{
  public:
	  FPBDCollisionSphereConstraints(const FPBDParticles& InParticles, const TSet<TVec2<int32>>& DisabledCollisionElements, const FReal Dt, const FReal Height = (FReal)0.)
	    : MH(Height)
	{
		for (int32 i = 0; i < (int32)InParticles.Size(); ++i)
		{
			MObjects.Add(Chaos::FImplicitObjectPtr(new FSphere(InParticles.P(i), Height)));
		}
		TBoundingVolumeHierarchy<TArray<Chaos::FImplicitObjectPtr>, TArray<int32>> Hierarchy(MObjects);
		FCriticalSection CriticalSection;
		PhysicsParallelFor(InParticles.Size(), [&](int32 Index) {
			TArray<int32> PotentialIntersections = Hierarchy.FindAllIntersections(InParticles.P(Index));
			for (int32 i = 0; i < PotentialIntersections.Num(); ++i)
			{
				int32 Index2 = PotentialIntersections[i];
				if (Index == Index2 || DisabledCollisionElements.Contains(TVector<int32, 2>(Index, Index2)))
					continue;
				if ((InParticles.P(Index2) - InParticles.P(Index)).Size() < Height)
				{
					CriticalSection.Lock();
					MConstraints.FindOrAdd(Index).Add(Index2);
					CriticalSection.Unlock();
				}
			}
		});
	}
	virtual ~FPBDCollisionSphereConstraints() = default;

	void Apply(FPBDParticles& InParticles, const FReal Dt, const int32 Index) const override
	{
		if (InParticles.InvM(Index) == 0 || !MConstraints.Contains(Index))
			return;
		for (int32 i = 0; i < MConstraints[Index].Num(); ++i)
		{
			FVec3 Normal;
			FReal Phi = MObjects[MConstraints[Index][i]]->PhiWithNormal(InParticles.P(Index), Normal);
			if (Phi < 0)
			{
				InParticles.P(Index) += -Phi * Normal;
			}
		}
	}

  private:
	FReal MH;
	TMap<int32, TArray<int32>> MConstraints;
	TArray<Chaos::FImplicitObjectPtr> MObjects;
};

template <typename T, int d>
using TPBDCollisionSphereConstraints UE_DEPRECATED(4.27, "Deprecated. this class is to be deleted, use FPBDCollisionSphereConstraints instead") = FPBDCollisionSphereConstraints;

}
#endif
