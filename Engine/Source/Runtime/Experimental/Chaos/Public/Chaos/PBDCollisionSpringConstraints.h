// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if !COMPILE_WITHOUT_UNREAL_SUPPORT
#include "Chaos/PBDCollisionSpringConstraintsBase.h"
#include "Chaos/CollectionPropertyFacade.h"

namespace Chaos::Softs
{

class FPBDCollisionSpringConstraints : public FPBDCollisionSpringConstraintsBase
{
	typedef FPBDCollisionSpringConstraintsBase Base;

public:
	static constexpr FSolverReal MinFrictionCoefficient = (FSolverReal)0.;
	static constexpr FSolverReal MaxFrictionCoefficient = (FSolverReal)1.;
	static constexpr int32 DefaultSelfCollisionDisableNeighborDistance = 5;

	static bool IsEnabled(const FCollectionPropertyConstFacade& PropertyCollection)
	{
		return GetUseSelfCollisions(PropertyCollection, false);
	}

	FPBDCollisionSpringConstraints(
		const int32 InOffset,
		const int32 InNumParticles,
		const FTriangleMesh& InTriangleMesh,
		const TArray<FSolverVec3>* InRestPositions,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
		const TMap<FString, TConstArrayView<int32>>& FaceIntMaps,
		const FCollectionPropertyConstFacade& PropertyCollection)
		: Base(
			InOffset,
			InNumParticles,
			InTriangleMesh,
			InRestPositions,
			GenerateDisabledCollisionElements(InTriangleMesh, PropertyCollection),
			WeightMaps.FindRef(GetSelfCollisionKinematicColliderFrictionString(PropertyCollection, SelfCollisionKinematicColliderFrictionName.ToString())),
			WeightMaps.FindRef(GetSelfCollisionThicknessString(PropertyCollection, SelfCollisionThicknessName.ToString())),
			FaceIntMaps.FindRef(GetSelfCollisionLayersString(PropertyCollection, SelfCollisionLayersName.ToString())),
			FSolverVec2::Max(FSolverVec2(GetWeightedFloatSelfCollisionThickness(PropertyCollection, Base::BackCompatThickness)), FSolverVec2(0.f)),
			(FSolverReal)FMath::Clamp(GetSelfCollisionStiffness(PropertyCollection, Base::BackCompatStiffness), 0.f, 1.f),
			FMath::Clamp((FSolverReal)GetSelfCollisionFriction(PropertyCollection, Base::BackCompatFrictionCoefficient), MinFrictionCoefficient, MaxFrictionCoefficient),
			GetSelfCollideAgainstKinematicCollidersOnly(PropertyCollection, false),
			(FSolverReal)FMath::Max(GetSelfCollisionKinematicColliderThickness(PropertyCollection, Base::DefaultKinematicColliderThickness), 0.f),
			(FSolverReal)FMath::Clamp((FSolverReal)GetSelfCollisionKinematicColliderStiffness(PropertyCollection, Base::DefaultKinematicColliderStiffness), 0.f, 1.f),
			FSolverVec2(GetWeightedFloatSelfCollisionKinematicColliderFriction(PropertyCollection, Base::DefaultKinematicColliderFrictionCoefficient)).ClampAxes(MinFrictionCoefficient, MaxFrictionCoefficient),
			(FSolverReal)GetSelfCollisionProximityStiffness(PropertyCollection, Base::DefaultProximityStiffness))
		, SelfCollisionThicknessIndex(PropertyCollection)
		, SelfCollisionStiffnessIndex(PropertyCollection)
		, SelfCollisionFrictionIndex(PropertyCollection)
		, SelfCollisionProximityStiffnessIndex(PropertyCollection)
		, SelfCollisionLayersIndex(PropertyCollection)
		, SelfCollideAgainstKinematicCollidersOnlyIndex(PropertyCollection)
		, SelfCollisionKinematicColliderThicknessIndex(PropertyCollection)
		, SelfCollisionKinematicColliderStiffnessIndex(PropertyCollection)
		, SelfCollisionKinematicColliderFrictionIndex(PropertyCollection)
	{}

	UE_DEPRECATED(5.4, "Use constructor with FaceIntMaps")
	FPBDCollisionSpringConstraints(
		const int32 InOffset,
		const int32 InNumParticles,
		const FTriangleMesh& InTriangleMesh,
		const TArray<FSolverVec3>* InRestPositions,
		TSet<TVec2<int32>>&& InDisabledCollisionElements,
		const FCollectionPropertyConstFacade& PropertyCollection)
		: FPBDCollisionSpringConstraints(
			InOffset,
			InNumParticles,
			InTriangleMesh,
			InRestPositions,
			TMap<FString, TConstArrayView<FRealSingle>>(),
			TMap<FString, TConstArrayView<int32>>(),
			PropertyCollection
		)
	{}

	FPBDCollisionSpringConstraints(
		const int32 InOffset,
		const int32 InNumParticles,
		const FTriangleMesh& InTriangleMesh,
		const TArray<FSolverVec3>* InRestPositions,
		TSet<TVec2<int32>>&& InDisabledCollisionElements,
		const FSolverReal InThickness = Base::BackCompatThickness,
		const FSolverReal InStiffness = Base::BackCompatStiffness,
		const FSolverReal InFrictionCoefficient = Base::BackCompatFrictionCoefficient)
		: Base(
			InOffset,
			InNumParticles,
			InTriangleMesh,
			InRestPositions,
			MoveTemp(InDisabledCollisionElements),
			TConstArrayView<FRealSingle>(),
			TConstArrayView<FRealSingle>(),
			TConstArrayView<int32>(),
			FSolverVec2(InThickness),
			InStiffness,
			InFrictionCoefficient)
		, SelfCollisionThicknessIndex(ForceInit)
		, SelfCollisionStiffnessIndex(ForceInit)
		, SelfCollisionFrictionIndex(ForceInit)
		, SelfCollisionProximityStiffnessIndex(ForceInit)
		, SelfCollisionLayersIndex(ForceInit)
		, SelfCollideAgainstKinematicCollidersOnlyIndex(ForceInit)
		, SelfCollisionKinematicColliderThicknessIndex(ForceInit)
		, SelfCollisionKinematicColliderStiffnessIndex(ForceInit)
		, SelfCollisionKinematicColliderFrictionIndex(ForceInit)
	{}

	virtual ~FPBDCollisionSpringConstraints() override = default;

	using Base::Init;

	void SetProperties(const FCollectionPropertyConstFacade& PropertyCollection,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
		const TMap<FString, TConstArrayView<int32>>& FaceIntMaps)
	{
		if (IsSelfCollisionThicknessMutable(PropertyCollection))
		{
			const FSolverVec2 WeightedValue = FSolverVec2::Max(FSolverVec2(GetWeightedFloatSelfCollisionThickness(PropertyCollection)), FSolverVec2(0.f));
			if (IsSelfCollisionThicknessStringDirty(PropertyCollection))
			{
				const FString& WeightMapName = GetSelfCollisionThicknessString(PropertyCollection);
				ThicknessWeighted = FPBDFlatWeightMap(WeightedValue, WeightMaps.FindRef(WeightMapName), GetNumParticles());
			}
			else
			{
				ThicknessWeighted.SetWeightedValue(WeightedValue);
			}
		}
		if (IsSelfCollisionStiffnessMutable(PropertyCollection))
		{
			Stiffness = (FSolverReal)FMath::Clamp(GetSelfCollisionStiffness(PropertyCollection), 0.f, 1.f);
		}
		if (IsSelfCollisionFrictionMutable(PropertyCollection))
		{
			FrictionCoefficient = FMath::Clamp((FSolverReal)GetSelfCollisionFriction(PropertyCollection), MinFrictionCoefficient, MaxFrictionCoefficient);
		}
		if (IsSelfCollisionProximityStiffnessMutable(PropertyCollection))
		{
			ProximityStiffness = GetSelfCollisionProximityStiffness(PropertyCollection);
		}
		if (IsSelfCollisionLayersMutable(PropertyCollection))
		{
			if (IsSelfCollisionLayersStringDirty(PropertyCollection))
			{
				const FString& FaceIntMapName = GetSelfCollisionLayersString(PropertyCollection);
				Base::UpdateCollisionLayers(FaceIntMaps.FindRef(FaceIntMapName));
			}
		}
		if (IsSelfCollideAgainstKinematicCollidersOnlyMutable(PropertyCollection))
		{
			bOnlyCollideKinematics = GetSelfCollideAgainstKinematicCollidersOnly(PropertyCollection);
		}
		if (IsSelfCollisionKinematicColliderThicknessMutable(PropertyCollection))
		{
			KinematicCollisions.SetColliderThickness(FMath::Max(GetSelfCollisionKinematicColliderThickness(PropertyCollection), 0.f));
		}
		if (IsSelfCollisionKinematicColliderStiffnessMutable(PropertyCollection))
		{
			KinematicCollisions.SetStiffness(FMath::Clamp((FSolverReal)GetSelfCollisionKinematicColliderStiffness(PropertyCollection), 0.f, 1.f));
		}
		if (IsSelfCollisionKinematicColliderFrictionMutable(PropertyCollection))
		{
			const FSolverVec2 WeightedValue = FSolverVec2(GetWeightedFloatSelfCollisionKinematicColliderFriction(PropertyCollection)).ClampAxes(MinFrictionCoefficient, MaxFrictionCoefficient);
			if (IsSelfCollisionKinematicColliderFrictionStringDirty(PropertyCollection))
			{
				const FString& WeightMapName = GetSelfCollisionKinematicColliderFrictionString(PropertyCollection);
				KinematicColliderFrictionCoefficient = FPBDFlatWeightMap(WeightedValue, WeightMaps.FindRef(WeightMapName), GetNumParticles());
			}
			else
			{
				KinematicColliderFrictionCoefficient.SetWeightedValue(WeightedValue);
			}
		}
	}

	UE_DEPRECATED(5.4, "Use SetProperties with FaceIntMaps")
	void SetProperties(const FCollectionPropertyConstFacade& PropertyCollection)
	{
		SetProperties(PropertyCollection, TMap<FString, TConstArrayView<FRealSingle>>(), TMap<FString, TConstArrayView<int32>>());
	}

	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(UseSelfCollisions, bool);
	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(SelfCollisionThickness, float);
	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(SelfCollisionStiffness, float);
	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(SelfCollisionFriction, float);
	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(SelfCollisionLayers, float);
private:
	using Base::ThicknessWeighted;
	using Base::Stiffness;
	using Base::FrictionCoefficient;
	using Base::ProximityStiffness;

	static TSet<TVec2<int32>> GenerateDisabledCollisionElements(
		const FTriangleMesh& TriangleMesh, const FCollectionPropertyConstFacade& PropertyCollection)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPBDCollisionSpringConstraints_GenerateDisabledCollisionElements);
		const int32 DisabledCollisionElementsN = GetSelfCollisionDisableNeighborDistance(PropertyCollection, DefaultSelfCollisionDisableNeighborDistance);
		TSet<TVec2<int32>> DisabledCollisionElements;  // TODO: Is this needed? Turn this into a bit array?

		const TVec2<int32> Range = TriangleMesh.GetVertexRange();
		for (int32 Index = Range[0]; Index <= Range[1]; ++Index)
		{
			const TSet<int32> Neighbors = TriangleMesh.GetNRing(Index, DisabledCollisionElementsN);
			for (int32 Element : Neighbors)
			{
				check(Index != Element);
				DisabledCollisionElements.Emplace(TVec2<int32>(Index, Element));
				DisabledCollisionElements.Emplace(TVec2<int32>(Element, Index));
			}
		}
		return DisabledCollisionElements;
	}

	UE_CHAOS_DECLARE_INDEXED_PROPERTYCOLLECTION_NAME(SelfCollisionThickness, float);
	UE_CHAOS_DECLARE_INDEXED_PROPERTYCOLLECTION_NAME(SelfCollisionStiffness, float);
	UE_CHAOS_DECLARE_INDEXED_PROPERTYCOLLECTION_NAME(SelfCollisionFriction, float);

	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(SelfCollisionProximityStiffness, float);
	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(SelfCollisionDisableNeighborDistance, int32);
	UE_CHAOS_DECLARE_INDEXED_PROPERTYCOLLECTION_NAME(SelfCollisionLayers, float);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(SelfCollideAgainstKinematicCollidersOnly, bool);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(SelfCollisionKinematicColliderThickness, float);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(SelfCollisionKinematicColliderStiffness, float);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(SelfCollisionKinematicColliderFriction, float);
};

}  // End namespace Chaos::Softs

#endif
