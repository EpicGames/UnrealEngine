// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if !COMPILE_WITHOUT_UNREAL_SUPPORT
#include "Chaos/Core.h"
#include "Chaos/PBDFlatWeightMap.h"
#include "Chaos/PBDSoftsEvolutionFwd.h"
#include "Chaos/PBDSoftsSolverParticles.h"
#include "Chaos/PBDKinematicTriangleMeshCollisions.h"
#include "Chaos/PBDTriangleMeshCollisions.h"
#include "Containers/Set.h"


namespace Chaos
{
	class FTriangleMesh;
}

namespace Chaos::Softs
{

// This is an invertible spring class, typical springs are not invertible aware
class FPBDCollisionSpringConstraintsBase
{
public:
	static constexpr FSolverReal BackCompatThickness = (FSolverReal)1.f;
	static constexpr FSolverReal BackCompatStiffness = (FSolverReal)0.5f;
	static constexpr FSolverReal BackCompatFrictionCoefficient = (FSolverReal)0.f;
	static constexpr FSolverReal DefaultKinematicColliderThickness = (FSolverReal)0.f;
	static constexpr FSolverReal DefaultKinematicColliderStiffness = (FSolverReal)1.f;
	static constexpr FSolverReal DefaultKinematicColliderFrictionCoefficient = (FSolverReal)0.f;
	static constexpr FSolverReal DefaultProximityStiffness = (FSolverReal)1.;

	CHAOS_API FPBDCollisionSpringConstraintsBase(
		const int32 InOffset,
		const int32 InNumParticles,
		const FTriangleMesh& InTriangleMesh,
		const TArray<FSolverVec3>* InReferencePositions,
		TSet<TVec2<int32>>&& InDisabledCollisionElements,
		const TConstArrayView<FRealSingle>& InThicknessMultipliers,
		const TConstArrayView<FRealSingle>& InKinematicColliderFrictionMultipliers,
		const TConstArrayView<int32>& InSelfCollisionLayers,
		const FSolverVec2 InThickness = FSolverVec2(BackCompatThickness),
		const FSolverReal InStiffness = BackCompatStiffness,
		const FSolverReal InFrictionCoefficient = BackCompatFrictionCoefficient,
		const bool bInOnlyCollideKinematics = false,
		const FSolverReal InKinematicColliderThickness = DefaultKinematicColliderThickness,
		const FSolverReal InKinematicColliderStiffness = DefaultKinematicColliderStiffness,
		const FSolverVec2 InKinematicColliderFrictionCoefficient = FSolverVec2(DefaultKinematicColliderFrictionCoefficient),
		const FSolverReal InProximityStiffness = DefaultProximityStiffness);

	virtual ~FPBDCollisionSpringConstraintsBase() = default;

	void ResetKinematicCollider()
	{
		KinematicCollisions.Reset();
	}

	template<typename SpatialAccelerator, typename SolverParticlesOrRange>
	UE_DEPRECATED(5.4, "Use Init with CollidableSubMesh. This method is much less efficient as it recreates the CollidableSubMesh each call.")
	CHAOS_API void Init(const SolverParticlesOrRange& Particles, const SpatialAccelerator& Spatial, const TConstArrayView<FPBDTriangleMeshCollisions::FGIAColor>& VertexGIAColors, const TArray<FPBDTriangleMeshCollisions::FGIAColor>& TriangleGIAColors);

	template<typename SpatialAccelerator, typename SolverParticlesOrRange>
	CHAOS_API void Init(const SolverParticlesOrRange& Particles, const FSolverReal Dt, const FPBDTriangleMeshCollisions::FTriangleSubMesh& CollidableSubMesh,
		const SpatialAccelerator& DynamicSpatial, const SpatialAccelerator& KinematicColliderSpatial, const TConstArrayView<FPBDTriangleMeshCollisions::FGIAColor>& VertexGIAColors, const TArray<FPBDTriangleMeshCollisions::FGIAColor>& TriangleGIAColors);

	template<typename SolverParticlesOrRange>
	CHAOS_API FSolverVec3 GetDelta(const SolverParticlesOrRange& InParticles, const int32 i) const;

	const TArray<TVec4<int32>>& GetConstraints() const 
	{
		return Constraints;  
	}
	const TArray<FSolverVec3>& GetBarys() const 
	{
		return Barys;
	}

	UE_DEPRECATED(5.4, "Thickness is now weighted.")
	FSolverReal GetThickness() const { return (FSolverReal)ThicknessWeighted; }

	FSolverReal GetMaxThickness() const { return FMath::Max(ThicknessWeighted.GetLow(), ThicknessWeighted.GetHigh()); }
	FSolverReal GetParticleThickness(int32 ParticleIndex) const { return ThicknessWeighted.GetValue(ParticleIndex - Offset); }
	const FPBDFlatWeightMap& GetThicknessWeighted() const { return ThicknessWeighted; }

	bool GetGlobalIntersectionAnalysis() const { return bGlobalIntersectionAnalysis; }
	const TArray<bool>& GetFlipNormals() const 
	{
		return FlipNormal;
	}

	const TArray<int32>& GetKinematicCollidingParticles() const { return KinematicCollisions.GetCollidingParticles(); }
	const TArray<TMap<int32, FSolverReal>>& GetKinematicColliderTimers() const { return KinematicCollisions.GetTimers(); }
	const FTriangleMesh* GetKinematicColliderTriangleMesh() const { return KinematicCollisions.GetTriangleMesh(); }
	const FTriangleMesh& GetTriangleMesh() const { return TriangleMesh; }

	void SetThicknessWeighted(const FSolverVec2 InThickness) { ThicknessWeighted.SetWeightedValue(FSolverVec2::Max(InThickness, FSolverVec2(0.f))); }
	void SetFrictionCoefficient(FSolverReal InFrictionCoefficient) { FrictionCoefficient = InFrictionCoefficient; }

	template<typename SolverParticlesOrRange>
	void Apply(SolverParticlesOrRange& Particles, const FSolverReal Dt, const int32 ConstraintIndex) const
	{
		const TVector<int32, 4>& Constraint = Constraints[ConstraintIndex];
		const int32 Index1 = Constraint[0];
		const int32 Index2 = Constraint[1];
		const int32 Index3 = Constraint[2];
		const int32 Index4 = Constraint[3];
		const FSolverVec3 Delta = GetDelta(Particles, ConstraintIndex);
		if (Particles.InvM(Index1) > 0)
		{
			Particles.P(Index1) += Particles.InvM(Index1) * Delta;
		}
		if (Particles.InvM(Index2) > (FSolverReal)0.)
		{
			Particles.P(Index2) -= Particles.InvM(Index2) * Barys[ConstraintIndex][0] * Delta;
		}
		if (Particles.InvM(Index3) > (FSolverReal)0.)
		{
			Particles.P(Index3) -= Particles.InvM(Index3) * Barys[ConstraintIndex][1] * Delta;
		}
		if (Particles.InvM(Index4) > (FSolverReal)0.)
		{
			Particles.P(Index4) -= Particles.InvM(Index4) * Barys[ConstraintIndex][2] * Delta;
		}
	}

	template<typename SolverParticlesOrRange>
	CHAOS_API void Apply(SolverParticlesOrRange& InParticles, const FSolverReal Dt) const;

	void Apply(FSolverParticles& InParticles, const FSolverReal Dt, const TArray<int32>& InConstraintIndices) const
	{
		for (int32 ConstraintIndex : InConstraintIndices)
		{
			Apply(InParticles, Dt, ConstraintIndex);
		}
	}

	CHAOS_API void UpdateLinearSystem(const FSolverParticlesRange& Particles, const FSolverReal Dt, FEvolutionLinearSystem& LinearSystem) const;
	
	TConstArrayView<int32> GetFaceCollisionLayers() const { return FaceCollisionLayers; }
	const TArray<TVector<int32, 2>>& GetVertexCollisionLayers() const { return VertexCollisionLayers; }

	FSolverReal GetConstraintThickness(const int32 ConstraintIndex) const
	{
		if (!ThicknessWeighted.HasWeightMap())
		{
			return 2.f * (FSolverReal)ThicknessWeighted;
		}
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		const TVec4<int32>& Constraint = Constraints[ConstraintIndex];
		const FSolverVec3& Bary = Barys[ConstraintIndex];
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		const int32 Index1 = Constraint[0] - Offset;
		const int32 Index2 = Constraint[1] - Offset;
		const int32 Index3 = Constraint[2] - Offset;
		const int32 Index4 = Constraint[3] - Offset;

		return ThicknessWeighted.GetValue(Index1) + Bary[0] * ThicknessWeighted.GetValue(Index2) + Bary[1] * ThicknessWeighted.GetValue(Index3) + Bary[2] * ThicknessWeighted.GetValue(Index4);
	}

	FSolverReal GetConstraintFrictionCoefficient(const int32 ConstraintIndex) const
	{
		switch (ConstraintTypes[ConstraintIndex])
		{
		default:
		case EConstraintType::Default:
			return FrictionCoefficient;
		case EConstraintType::GIAFlipped:
			return (FSolverReal)0.f;
		}
	}

protected:
	FPBDFlatWeightMap ThicknessWeighted;
	FSolverReal Stiffness; // (0-1 compliance for PBD)
	FSolverReal FrictionCoefficient; 
	bool bOnlyCollideKinematics;

	FPBDFlatWeightMap KinematicColliderFrictionCoefficient;
	FPBDKinematicTriangleMeshCollisions KinematicCollisions;

	UE_DEPRECATED(5.6, "Use KinematicCollisions.Thickness instead")
	FSolverReal KinematicColliderThickness;
	UE_DEPRECATED(5.6, "Use KinematicCollisions.Stiffness instead")
	FSolverReal KinematicColliderStiffness;

	FSolverReal ProximityStiffness; // (actual spring stiffness for force-based solver)

	CHAOS_API void UpdateCollisionLayers(const TConstArrayView<int32>& InFaceCollisionLayers);

	int32 GetNumParticles() const { return NumParticles; }

private:

	TArray<TVec4<int32>> Constraints;
	TArray<FSolverVec3> Barys;
	TArray<bool> FlipNormal;

	template<typename SolverParticlesOrRange>
	void ApplyDynamicConstraints(SolverParticlesOrRange& InParticles, const FSolverReal Dt) const;

	const FTriangleMesh& TriangleMesh;
	const TArray<FSolverVec3>* ReferencePositions;
	const TSet<TVec2<int32>> DisabledCollisionElements;  // TODO: Make this a bitarray
	TConstArrayView<int32> FaceCollisionLayers;
	TArray<TVector<int32, 2>> VertexCollisionLayers; // Only non-empty if FaceCollisionLayers is non-empty. Values are Min and Max layers for that vertex

	enum struct EConstraintType : uint8
	{
		Default,
		GIAFlipped,
	};
	TArray<EConstraintType> ConstraintTypes;


	int32 Offset;
	int32 NumParticles;
	bool bGlobalIntersectionAnalysis; // This is set based on which Init is called.
};
}  // End namespace Chaos::Softs

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#if !defined(CHAOS_COLLISION_SPRING_ISPC_ENABLED_DEFAULT)
#define CHAOS_COLLISION_SPRING_ISPC_ENABLED_DEFAULT 1
#endif
#endif

UE_DEPRECATED(5.6, "This variable has been renamed bChaos_KinematicTriangleMesh_ISPC_Enabled")
static constexpr bool bChaos_CollisionSpring_ISPC_Enabled = false;

#endif
