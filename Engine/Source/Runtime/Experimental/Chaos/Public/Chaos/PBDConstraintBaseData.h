// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Chaos/Core.h"
#include "Chaos/Transform.h"
#include "Chaos/Vector.h"

#include "Chaos/Framework/PhysicsProxyBase.h"
#include "Chaos/Framework/PhysicsSolverBase.h"
#include "Properties.h"

namespace Chaos
{

enum class EConstraintType
{
	NoneType = 0,
	JointConstraintType = 1,
	SpringConstraintType = 2,
	SuspensionConstraintType = 3,
	CharacterGroundConstraintType = 4,
};

#define CONSTRAINT_JOINT_PROPERPETY_IMPL(TYPE, FNAME, ENAME, VNAME)\
	void Set##FNAME(TYPE InValue){ if (InValue != VNAME){VNAME = InValue;MDirtyFlags.MarkDirty(ENAME);SetProxy(Proxy);}}\
	TYPE Get##FNAME() const{return VNAME;}\

#define CONSTRAINT_JOINT_PROPERPETY_IMPL2(TYPE, FNAME, PROP, VNAME)\
	void Set##FNAME(TYPE InValue){ PROP.Modify(/*bInvalidate=*/true, DirtyFlags, Proxy, [&InValue](auto& Data) { Data.VNAME = InValue; }); }\
	TYPE Get##FNAME() const{return PROP.Read().VNAME;}\

class FConstraintBase
{
public:
	virtual ~FConstraintBase() {}

	typedef TVector<TGeometryParticleHandle<FReal, 3>*, 2> FParticleHandlePair;

	CHAOS_API FConstraintBase(EConstraintType InType);

	EConstraintType GetType() const { return Type; }
	bool IsType(EConstraintType InType) { return (Type == InType); }
	CHAOS_API bool IsValid() const;

	bool IsDirty() const { return DirtyFlags.IsDirty(); }
	bool IsDirty(const EChaosPropertyFlags CheckBits) const { return DirtyFlags.IsDirty(CheckBits); }
	void ClearDirtyFlags() { DirtyFlags.Clear(); }
	const FDirtyChaosPropertyFlags& GetDirtyFlags() const { return DirtyFlags; }

	//template <typename CastType>
	//static CastType CastTo<CastType>(EConstraintType InType) { (InType == Type) ? static_cast<CastType*> : nullptr; }

	template<typename T = IPhysicsProxyBase> T* GetProxy() { return static_cast<T*>(Proxy); }

	CHAOS_API void SetProxy(IPhysicsProxyBase* InProxy);

	void SyncRemoteData(FDirtyPropertiesManager& Manager, int32 DataIdx, FDirtyChaosProperties& RemoteData)
	{
		RemoteData.SetFlags(DirtyFlags);
		SyncRemoteDataImp(Manager, DataIdx, RemoteData);
		ClearDirtyFlags();
	}

protected:
	EConstraintType Type;
	class IPhysicsProxyBase* Proxy;
	FDirtyChaosPropertyFlags DirtyFlags;

	virtual void SyncRemoteDataImp(FDirtyPropertiesManager& Manager, int32 DataIdx, FDirtyChaosProperties& RemoteData) = 0;
};

} // Chaos

