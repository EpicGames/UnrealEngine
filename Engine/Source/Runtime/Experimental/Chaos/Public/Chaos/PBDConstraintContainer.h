// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/ConstraintHandle.h"
#include "Chaos/Transform.h"
#include "Chaos/Evolution/SolverBody.h"
#include "Chaos/Evolution/SolverConstraintContainer.h"
#include "Chaos/Evolution/ConstraintGroupSolver.h"
#include "Chaos/Island/IslandManagerFwd.h"

namespace Chaos
{
	/**
	 * Base class for containers of constraints.
	 * A Constraint Container holds an array of constraints and provides methods to allocate and deallocate constraints
	 *as well as the API required to plug into Constraint Rules.
	 */
	class FPBDConstraintContainer
	{
	public:
		CHAOS_API FPBDConstraintContainer(FConstraintHandleTypeID InConstraintHandleType);

		CHAOS_API virtual ~FPBDConstraintContainer();

		/**
		 * The ContainerId is used to map constraint handles back to their constraint container.
		 * Every container is assigned an ID when it is registered - it will be the array index in the Evolution
		*/
		int32 GetContainerId() const
		{
			return ContainerId;
		}

		/**
		 * @see GetContainerId()
		*/
		void SetContainerId(int32 InContainerId)
		{
			ContainerId = InContainerId;
		}

		/**
		 * The TypeID of the constraints in this container. Used to safely downcast constraint handles.
		 * @see FConstraintHandle::As()
		*/
		const FConstraintHandleTypeID& GetConstraintHandleType() const
		{
			return ConstraintHandleType;
		}

		/**
		 * Get the number of constraints in this container (includes inactive and disabled)
		*/
		virtual int32 GetNumConstraints() const = 0;

		/**
		 * Empty the constraints (must be removed from the graph first, if required)
		*/
		virtual void ResetConstraints() = 0;

		/**
		 * An opportunity to create/destroy constraints based on particle state.
		*/
		virtual void UpdatePositionBasedState(const FReal Dt) {}

		/**
		 * Called oncer per tick to initialize buffers required for the rest of the tick
		*/
		virtual void PrepareTick() = 0;

		/**
		 * Should undo any allocations in PrepareTick
		*/
		virtual void UnprepareTick() = 0;

		// This is called when a particle is destroyed. The constraint should be disabled and the particle in the constraint set to null.
		// NOTE: This should only act on persistent constraints (e.g. joints) and is a no-op for transient constraints (collisions)
		// @todo(chaos): remove the set
		virtual void DisconnectConstraints(const TSet<TGeometryParticleHandle<FReal, 3>*>&) {}

		// This is called to notify the constraint container that a particle has been disabled
		virtual void OnDisableParticle(FGeometryParticleHandle* DisabledParticle);

		// This is called to notify the constraint container that a particle has been disabled
		virtual void OnEnableParticle(FGeometryParticleHandle* EnabledParticle);

		/**
		* Create a constraint solver for an Evolution without Graph support (RBAN evolution).
		* There will only be one of these per scene (RBAN node) and it is used to solve all constraints
		* in the container (serially).
		*/
		virtual TUniquePtr<FConstraintContainerSolver> CreateSceneSolver(const int32 Priority) = 0;

		/**
		 * Create a constraint solver for an Evolution with Graph support (World evolution).
		 * The system will create several of these: usually one per worker thread (Island Group) but possibly
		 * more in complex scenes where constraint coloring is being used. It will be used to solve constraints
		 * in groups, with the constraints in each group determined by the graph/islands/islandgroups.
		*/
		virtual TUniquePtr<FConstraintContainerSolver> CreateGroupSolver(const int32 Priority) = 0;

		/**
		 * Add all the constraints in the container to the graph
		*/
		virtual void AddConstraintsToGraph(Private::FPBDIslandManager& IslandManager) = 0;

	protected:
		FConstraintHandleTypeID ConstraintHandleType;
		int32 ContainerId;
	};

	//
	//
	// From ConstraintHandle.h
	//
	//

	inline int32 FConstraintHandle::GetContainerId() const
	{
		if (ConstraintContainer != nullptr)
		{
			return ConstraintContainer->GetContainerId();
		}
		return INDEX_NONE;
	}

	inline const FConstraintHandleTypeID& FConstraintHandle::GetType() const
	{
		if (ConstraintContainer != nullptr)
		{
			return ConstraintContainer->GetConstraintHandleType();
		}
		return FConstraintHandle::InvalidType();
	}

	template<typename T>
	inline T* FConstraintHandle::As()
	{
		// @todo(chaos): we need a safe cast that allows for casting to intermediate base classes (e.g., FIndexedConstraintHandle)
		return ((ConstraintContainer != nullptr) && ConstraintContainer->GetConstraintHandleType().IsA(T::StaticType())) ? static_cast<T*>(this) : nullptr;
	}

	template<typename T>
	inline const T* FConstraintHandle::As() const
	{
		return ((ConstraintContainer != nullptr) && ConstraintContainer->GetConstraintHandleType().IsA(T::StaticType())) ? static_cast<const T*>(this) : nullptr;
	}


}
