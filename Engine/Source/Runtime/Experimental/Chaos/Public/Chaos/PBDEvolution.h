// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/ArrayCollection.h"
#include "Chaos/PBDActiveView.h"
#include "Chaos/PBDSoftsEvolutionFwd.h"
#include "Chaos/PBDSoftsSolverParticles.h"
#include "Chaos/SoftsSolverCollisionParticles.h"
#include "Chaos/VelocityField.h"

namespace Chaos::Softs
{

class FPBDEvolution : public TArrayCollection
{
 public:
	// TODO: Tidy up this constructor (and update Headless Chaos)
	CHAOS_API FPBDEvolution(
		FSolverParticles&& InParticles,
		FSolverCollisionParticles&& InGeometryParticles,
		TArray<TVec3<int32>>&& CollisionTriangles,
		int32 NumIterations = 1,
		FSolverReal CollisionThickness = (FSolverReal)0.,
		FSolverReal SelfCollisionsThickness = (FSolverReal)0.,
		FSolverReal CoefficientOfFriction = (FSolverReal)0.,
		FSolverReal Damping = (FSolverReal)0.04,
		FSolverReal LocalDamping = (FSolverReal)0.,
		bool bDoQuasistatics = false, 
		bool InbUsePerParticleDamping = false);
	~FPBDEvolution() = default;

	// Advance one time step. Filter the input time step if specified.
	CHAOS_API void AdvanceOneTimeStep(const FSolverReal Dt);

	// Remove all particles, will also reset all rules
	CHAOS_API void ResetParticles();

	// Add particles and initialize group ids. Return the index of the first added particle.
	CHAOS_API int32 AddParticleRange(int32 NumParticles, uint32 GroupId, bool bActivate);

	// Return the number of particles of the block starting at Offset
	int32 GetParticleRangeSize(int32 Offset) const { return MParticlesActiveView.GetRangeSize(Offset); }

	// Set a block of particles active or inactive, using the index of the first added particle to identify the block.
	void ActivateParticleRange(int32 Offset, bool bActivate)  { MParticlesActiveView.ActivateRange(Offset, bActivate); }

	// Clear all ranges.
	void DeactivateParticleRanges() { MParticlesActiveView.Reset(0); }

	// Set a block of particles active or inactive, using the index of the first added particle to identify the block.
	int32 AddParticleRange(int32 NumItems, bool bActivate = true) { return MParticlesActiveView.AddRange(NumItems, bActivate); }


	// Particles accessors
	const FSolverParticles& Particles() const { return MParticles; }
	FSolverParticles& Particles() { return MParticles; }
	const TPBDActiveView<FSolverParticles>& ParticlesActiveView() { return MParticlesActiveView; }
	// These versions just help share code with Softs::FEvolution which follows UE naming standards.
	const FSolverParticles& GetParticles() const { return MParticles; }
	FSolverParticles& GetParticles() { return MParticles; }


	const TArray<uint32>& ParticleGroupIds() const { return MParticleGroupIds; }

	// Remove all collision particles
	CHAOS_API void ResetCollisionParticles(int32 NumParticles = 0);

	// Add collision particles and initialize group ids. Return the index of the first added particle.
	// Use INDEX_NONE as GroupId for collision particles that affect all particle groups.
	CHAOS_API int32 AddCollisionParticleRange(int32 NumParticles, uint32 GroupId, bool bActivate);

	// Add a single collision body particle to the solver. 
	CHAOS_API int32 AddCollisionParticle(uint32 GroupId, bool bActivate);

	// Remove a collision body from the active view list, and save its particle in the remvoed collision particle list
	CHAOS_API void RemoveCollisionParticle(int32 CollisionParticleIndex, int32 CollisionParticleViewIndex);

	// Set a block of collision particles active or inactive, using the index of the first added particle to identify the block.
	void ActivateCollisionParticleRange(int32 Offset, bool bActivate) { MCollisionParticlesActiveView.ActivateRange(Offset, bActivate); }

	// Return the number of particles of the block starting at Offset
	int32 GetCollisionParticleRangeSize(int32 Offset) const { return MCollisionParticlesActiveView.GetRangeSize(Offset); }

	// Collision particles accessors
	const FSolverCollisionParticles& CollisionParticles() const { return MCollisionParticles; }
	FSolverCollisionParticles& CollisionParticles() { return MCollisionParticles; }
	TArray<uint32>& CollisionParticleGroupIds() { return MCollisionParticleGroupIds; }
	const TArray<uint32>& CollisionParticleGroupIds() const { return MCollisionParticleGroupIds; }
	TPBDActiveView<FSolverCollisionParticles>& CollisionParticlesActiveView() { return MCollisionParticlesActiveView; }
	const TPBDActiveView<FSolverCollisionParticles>& CollisionParticlesActiveView() const { return MCollisionParticlesActiveView; }

	// Reset all constraint init and rule functions.
	void ResetConstraintRules() 
	{ 
		MConstraintInits.Reset(); 
		MConstraintRules.Reset(); 
		MPostCollisionConstraintRules.Reset();
		MConstraintPostprocessings.Reset();
		MConstraintInitsActiveView.Reset(); 
		MConstraintRulesActiveView.Reset();  
		MPostCollisionConstraintRulesActiveView.Reset();
		MConstraintPostprocessingsActiveView.Reset();
	}

	// Add constraints. Return the index of the first added constraint.
	CHAOS_API int32 AddConstraintInitRange(int32 NumConstraints, bool bActivate);
	CHAOS_API int32 AddConstraintRuleRange(int32 NumConstraints, bool bActivate);
	CHAOS_API int32 AddPostCollisionConstraintRuleRange(int32 NumConstraints, bool bActivate);
	CHAOS_API int32 AddConstraintPostprocessingsRange(int32 NumConstraints, bool bActivate);
	
	// Return the number of particles of the block starting at Offset
	int32 GetConstraintInitRangeSize(int32 Offset) const { return MConstraintInitsActiveView.GetRangeSize(Offset); }
	int32 GetConstraintRuleRangeSize(int32 Offset) const { return MConstraintRulesActiveView.GetRangeSize(Offset); }
	int32 GetPostCollisionConstraintRuleRangeSize(int32 Offset) const { return MPostCollisionConstraintRulesActiveView.GetRangeSize(Offset); }
	int32 GetConstraintPostprocessingsRangeSize(int32 Offset) const { return MConstraintPostprocessingsActiveView.GetRangeSize(Offset); }

	// Set a block of constraints active or inactive, using the index of the first added particle to identify the block.
	void ActivateConstraintInitRange(int32 Offset, bool bActivate) { MConstraintInitsActiveView.ActivateRange(Offset, bActivate); }
	void ActivateConstraintRuleRange(int32 Offset, bool bActivate) { MConstraintRulesActiveView.ActivateRange(Offset, bActivate); }
	void ActivatePostCollisionConstraintRuleRange(int32 Offset, bool bActivate) { MPostCollisionConstraintRulesActiveView.ActivateRange(Offset, bActivate); }
	void ActivateConstraintPostprocessingsRange(int32 Offset, bool bActivate) { MConstraintPostprocessingsActiveView.ActivateRange(Offset, bActivate); }

	// Constraint accessors
	const TArray<TFunction<void(FSolverParticles&, const FSolverReal)>>& ConstraintInits() const { return MConstraintInits; }
	TArray<TFunction<void(FSolverParticles&, const FSolverReal)>>& ConstraintInits() { return MConstraintInits; }
	const TArray<TFunction<void(FSolverParticles&, const FSolverReal)>>& ConstraintRules() const { return MConstraintRules; }
	TArray<TFunction<void(FSolverParticles&, const FSolverReal)>>& ConstraintRules() { return MConstraintRules; }
	const TArray<TFunction<void(FSolverParticles&, const FSolverReal)>>& PostCollisionConstraintRules() const { return MPostCollisionConstraintRules; }
	TArray<TFunction<void(FSolverParticles&, const FSolverReal)>>& PostCollisionConstraintRules() { return MPostCollisionConstraintRules; }
	const TArray<TFunction<void(FSolverParticles&, const FSolverReal)>>& ConstraintPostprocessings() const { return MConstraintPostprocessings; }
	TArray<TFunction<void(FSolverParticles&, const FSolverReal)>>& ConstraintPostprocessings() { return MConstraintPostprocessings; }
	
	void SetKinematicUpdateFunction(TFunction<void(FSolverParticles&, const FSolverReal, const FSolverReal, const int32)> KinematicUpdate) { MKinematicUpdate = KinematicUpdate; }
	void SetCollisionKinematicUpdateFunction(TFunction<void(FSolverCollisionParticles&, const FSolverReal, const FSolverReal, const int32)> KinematicUpdate) { MCollisionKinematicUpdate = KinematicUpdate; }

	TFunction<void(FSolverParticles&, const FSolverReal, const int32)>& GetForceFunction(const uint32 GroupId = 0) { return MGroupForceRules[GroupId]; }
	const TFunction<void(FSolverParticles&, const FSolverReal, const int32)>& GetForceFunction(const uint32 GroupId = 0) const { return MGroupForceRules[GroupId]; }

	const FSolverVec3& GetGravity(const uint32 GroupId = 0) const { check(GroupId < TArrayCollection::Size()); return MGroupGravityAccelerations[GroupId]; }
	void SetGravity(const FSolverVec3& Acceleration, const uint32 GroupId = 0) { check(GroupId < TArrayCollection::Size()); MGroupGravityAccelerations[GroupId] = Acceleration; }

	void SetQuasistatics(const bool bDoQuasistaticsIn) { bDoQuasistatics = bDoQuasistaticsIn; }

	FVelocityAndPressureField& GetVelocityAndPressureField(const uint32 GroupId = 0) { check(GroupId < TArrayCollection::Size()); return MGroupVelocityAndPressureFields[GroupId]; }
	const FVelocityAndPressureField& GetVelocityAndPressureField(const uint32 GroupId = 0) const { check(GroupId < TArrayCollection::Size()); return MGroupVelocityAndPressureFields[GroupId]; }

	int32 GetIterations() const { return MNumIterations; }
	void SetIterations(const int32 Iterations) { MNumIterations = Iterations; }

	FSolverReal GetCollisionThickness(const uint32 GroupId = 0) const { check(GroupId < TArrayCollection::Size()); return MGroupCollisionThicknesses[GroupId]; }
	void SetCollisionThickness(const FSolverReal CollisionThickness, const uint32 GroupId = 0) { check(GroupId < TArrayCollection::Size()); MGroupCollisionThicknesses[GroupId] = CollisionThickness; }

	FSolverReal GetCoefficientOfFriction(const uint32 GroupId = 0) const { check(GroupId < TArrayCollection::Size()); return MGroupCoefficientOfFrictions[GroupId]; }
	void SetCoefficientOfFriction(const FSolverReal CoefficientOfFriction, const uint32 GroupId = 0) { check(GroupId < TArrayCollection::Size()); MGroupCoefficientOfFrictions[GroupId] = CoefficientOfFriction; }

	FSolverReal GetDamping(const uint32 GroupId = 0) const { check(GroupId < TArrayCollection::Size()); return MGroupDampings[GroupId]; }
	void SetDamping(const FSolverReal Damping, const uint32 GroupId = 0) { check(GroupId < TArrayCollection::Size()); MGroupDampings[GroupId] = Damping; }

	FSolverReal GetLocalDamping(const uint32 GroupId = 0) const { check(GroupId < TArrayCollection::Size()); return MGroupLocalDampings[GroupId]; }
	void SetLocalDamping(const FSolverReal LocalDamping, const uint32 GroupId = 0) { check(GroupId < TArrayCollection::Size()); MGroupLocalDampings[GroupId] = LocalDamping; }

	bool GetUseCCD(const uint32 GroupId = 0) const { check(GroupId < TArrayCollection::Size()); return MGroupUseCCDs[GroupId]; }
	void SetUseCCD(const bool bUseCCD, const uint32 GroupId = 0) { check(GroupId < TArrayCollection::Size()); MGroupUseCCDs[GroupId] = bUseCCD; }

	FSolverReal GetParticleDamping(const uint32 ParticleIndex = 0) const { check(ParticleIndex < this->Particles().Size() && bUsePerParticleDamping); return MParticleDampings[ParticleIndex]; }
	void SetParticleDamping(const FSolverReal Damping, const uint32 ParticleIndex = 0) { check(ParticleIndex < this->Particles().Size()); if (bUsePerParticleDamping) { MParticleDampings[ParticleIndex] = Damping; } }

	const TArray<bool>& GetCollisionStatus() { return MCollided; }
	const TArray<FSolverVec3>& GetCollisionContacts() const { return MCollisionContacts; }
	const TArray<FSolverVec3>& GetCollisionNormals() const { return MCollisionNormals; }
	const TArray<FSolverReal>& GetCollisionPhis() const { return MCollisionPhis; }

	FSolverReal GetTime() const { return MTime; }

 private:
	// Add simulation groups and set default values
	CHAOS_API void AddGroups(int32 NumGroups);
	// Reset simulation groups
	CHAOS_API void ResetGroups();
	// Selected versions of the pre-iteration updates (euler step, force, velocity field. damping updates)..
	template<bool bForceRule, bool bVelocityField, bool bDampVelocityRule>
	void PreIterationUpdate(const FSolverReal Dt, const int32 Offset, const int32 Range, const int32 MinParallelBatchSize);

	struct FParticleVievToken { int32 ParticleIndex = INDEX_NONE; int32 ViewIndex = INDEX_NONE; };

private:
	FSolverParticles MParticles;
	TPBDActiveView<FSolverParticles> MParticlesActiveView;
	FSolverCollisionParticles MCollisionParticles;
	TPBDActiveView<FSolverCollisionParticles> MCollisionParticlesActiveView;
	TArray<FParticleVievToken> RemovedCollisionIndices;

	TArrayCollectionArray<FSolverRigidTransform3> MCollisionTransforms;  // Used for CCD to store the initial state before the kinematic update
	TArrayCollectionArray<bool> MCollided;
	TArrayCollectionArray<uint32> MCollisionParticleGroupIds;  // Used for per group parameters for collision particles
	TArrayCollectionArray<uint32> MParticleGroupIds;  // Used for per group parameters for particles
	TArray<FSolverVec3> MCollisionContacts;
	TArray<FSolverVec3> MCollisionNormals;
	TArray<FSolverReal> MCollisionPhis;

	TArrayCollectionArray<FSolverVec3> MGroupGravityAccelerations;
	TArrayCollectionArray<FVelocityAndPressureField> MGroupVelocityAndPressureFields;
	TArrayCollectionArray<TFunction<void(FSolverParticles&, const FSolverReal, const int32)>> MGroupForceRules;
	TArrayCollectionArray<FSolverReal> MGroupCollisionThicknesses;
	TArrayCollectionArray<FSolverReal> MGroupCoefficientOfFrictions;
	TArrayCollectionArray<FSolverReal> MGroupDampings;
	TArrayCollectionArray<FSolverReal> MGroupLocalDampings;
	TArrayCollectionArray<bool> MGroupUseCCDs;

	//per particle data:
	TArrayCollectionArray<FSolverReal> MParticleDampings;
	
	TArray<TFunction<void(FSolverParticles&, const FSolverReal)>> MConstraintInits;
	TPBDActiveView<TArray<TFunction<void(FSolverParticles&, const FSolverReal)>>> MConstraintInitsActiveView;
	TArray<TFunction<void(FSolverParticles&, const FSolverReal)>> MConstraintRules;
	TPBDActiveView<TArray<TFunction<void(FSolverParticles&, const FSolverReal)>>> MConstraintRulesActiveView;
	TArray<TFunction<void(FSolverParticles&, const FSolverReal)>> MPostCollisionConstraintRules;
	TPBDActiveView<TArray<TFunction<void(FSolverParticles&, const FSolverReal)>>> MPostCollisionConstraintRulesActiveView;
	TArray<TFunction<void(FSolverParticles&, const FSolverReal)>> MConstraintPostprocessings;
	TPBDActiveView<TArray<TFunction<void(FSolverParticles&, const FSolverReal)>>> MConstraintPostprocessingsActiveView;

	TFunction<void(FSolverParticles&, const FSolverReal, const FSolverReal, const int32)> MKinematicUpdate;
	TFunction<void(FSolverCollisionParticles&, const FSolverReal, const FSolverReal, const int32)> MCollisionKinematicUpdate;

	int32 MNumIterations;
	FSolverVec3 MGravity;
	FSolverReal MCollisionThickness;
	FSolverReal MCoefficientOfFriction;
	FSolverReal MDamping;
	FSolverReal MLocalDamping;
	FSolverReal MTime;
	bool bDoQuasistatics = false;
	bool bUsePerParticleDamping = false;
};

}  // End namespace Chaos::Softs

#if !defined(CHAOS_POST_ITERATION_UPDATES_ISPC_ENABLED)
#define CHAOS_POST_ITERATION_UPDATES_ISPC_ENABLED 1
#endif

// Support ISPC enable/disable in non-shipping builds
#if !INTEL_ISPC || UE_BUILD_SHIPPING
static constexpr bool bChaos_PostIterationUpdates_ISPC_Enabled = INTEL_ISPC && CHAOS_POST_ITERATION_UPDATES_ISPC_ENABLED;
#else
extern CHAOS_API bool bChaos_PostIterationUpdates_ISPC_Enabled;
#endif
