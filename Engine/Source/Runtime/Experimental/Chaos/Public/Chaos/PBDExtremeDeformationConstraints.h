// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/CollectionPropertyFacade.h"
#include "Chaos/PBDSoftsSolverParticles.h"
#include "Chaos/SoftsSolverParticlesRange.h"
#include "Chaos/Utilities.h"
#include "Containers/Array.h"

namespace Chaos::Softs
{

class FPBDExtremeDeformationConstraints
{
public:

	static bool IsEnabled(const FCollectionPropertyConstFacade& PropertyCollection)
	{
		return IsExtremeDeformationVertexSelectionEnabled(PropertyCollection, false) && 
			IsExtremeDeformationEdgeRatioThresholdEnabled(PropertyCollection, false);
	}

	template<int32 Valence UE_REQUIRES(Valence >= 2 && Valence <= 4)>
	FPBDExtremeDeformationConstraints(
		const FSolverParticlesRange& Particles,
		const TArray<TVector<int32, Valence>>& InConstraints,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
		const TMap<FString, const TSet<int32>*>& VertexSets,
		const FCollectionPropertyConstFacade& PropertyCollection,
		bool bTrimKinematicConstraints = false)
		: VertexSet(VertexSets.FindRef(GetExtremeDeformationVertexSelectionString(PropertyCollection, ExtremeDeformationVertexSelectionName.ToString()), nullptr))
		, Constraints(TrimConstraints(InConstraints,
			[&Particles, bTrimKinematicConstraints, this](int32 Index0, int32 Index1)
			{
				return (bTrimKinematicConstraints && Particles.InvM(Index0) == (FSolverReal)0. && Particles.InvM(Index1) == (FSolverReal)0.)
					|| (VertexSet && (!VertexSet->Contains(Index0) || !VertexSet->Contains(Index1)));
			}))
		, ParticleOffset(0)
		, ParticleCount(Particles.GetRangeSize())
		, ExtremeDeformationThreshold(GetExtremeDeformationEdgeRatioThreshold(PropertyCollection, FLT_MAX))
		, ExtremeDeformationEdgeRatioThresholdIndex(PropertyCollection)
	{
		// Update distances
		Dists.Reset(Constraints.Num());
		for (const TVec2<int32>& Constraint : Constraints)
		{
			const FSolverVec3& P0 = Particles.X(Constraint[0]);
			const FSolverVec3& P1 = Particles.X(Constraint[1]);
			Dists.Add((P1 - P0).Size());
		}
	}

	template<int32 Valence UE_REQUIRES(Valence >= 2 && Valence <= 4)>
	FPBDExtremeDeformationConstraints(
		const FSolverParticles& Particles,
		int32 InParticleOffset,
		int32 InParticleCount,
		const TArray<TVector<int32, Valence>>& InConstraints,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
		const TMap<FString, const TSet<int32>*>& VertexSets,
		const FCollectionPropertyConstFacade& PropertyCollection,
		bool bTrimKinematicConstraints = false)
		: VertexSet(VertexSets.FindRef(GetExtremeDeformationVertexSelectionString(PropertyCollection, ExtremeDeformationVertexSelectionName.ToString()), nullptr))
		, Constraints(TrimConstraints(InConstraints,
			[&Particles, bTrimKinematicConstraints, this](int32 Index0, int32 Index1)
			{
				return (bTrimKinematicConstraints && Particles.InvM(Index0) == (FSolverReal)0. && Particles.InvM(Index1) == (FSolverReal)0.)
					|| (VertexSet && (!VertexSet->Contains(Index0) || !VertexSet->Contains(Index1)));
			}))
		, ParticleOffset(InParticleOffset)
		, ParticleCount(InParticleCount)
		, ExtremeDeformationThreshold(GetExtremeDeformationEdgeRatioThreshold(PropertyCollection, FLT_MAX))
		, ExtremeDeformationEdgeRatioThresholdIndex(PropertyCollection)
	{
		// Update distances
		Dists.Reset(Constraints.Num());
		for (const TVec2<int32>& Constraint : Constraints)
		{
			const FSolverVec3& P0 = Particles.GetX(Constraint[0]);
			const FSolverVec3& P1 = Particles.GetX(Constraint[1]);
			Dists.Add((P1 - P0).Size());
		}
	}

	virtual ~FPBDExtremeDeformationConstraints() = default;

	void SetProperties(const FCollectionPropertyConstFacade& PropertyCollection,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps)
	{
		if (IsExtremeDeformationEdgeRatioThresholdMutable(PropertyCollection))
		{
			ExtremeDeformationThreshold = GetExtremeDeformationEdgeRatioThreshold(PropertyCollection, FLT_MAX);
		}
	}

	const TArray<TVec2<int32>>& GetConstraints() const { return Constraints; }

	CHAOS_API FSolverReal GetThreshold() const;
	
	/**
	* Returns if edges (after being pruned during initialization) are deformed above threshold compared to the rest positions.
	* @param Positions The solver particle positions, e.g. ClothingSimulationSolver.GetParticleXs().
	* @param Threshold edge ratio threshold to trigger detection.
	*/
	CHAOS_API bool IsExtremelyDeformed(TConstArrayView<Softs::FSolverVec3> Positions) const;

	/**
	* Returns if edges (after being pruned during initialization) are deformed above threshold compared to the reference positions.
	* @param Positions The solver particle positions, e.g. ClothingSimulationSolver.GetParticleXs().
	* @param ReferencePositions The reference particle positions, e.g. ClothingSimulationSolver.GetAnimationPositions(ParticleRangeId) if using the animation pose as reference.
	* @param Threshold edge ratio threshold to trigger detection.
	*/
	CHAOS_API bool IsExtremelyDeformed(TConstArrayView<Softs::FSolverVec3> Positions, const Softs::FSolverVec3* const RefencePositions) const;

	/**
	* Returns all edges (after being pruned during initialization) that are deformed above threshold compared to the rest positions.
	* @param Positions The solver particle positions, e.g. ClothingSimulationSolver.GetParticleXs().
	* @param Threshold edge ratio threshold to trigger detection.
	*/
	CHAOS_API TArray<TVec2<int32>> GetExtremelyDeformedEdges(TConstArrayView<Softs::FSolverVec3> Positions) const;

	/**
	* Returns edges that are deformed above threshold compared to reference positions.
	* @param Positions The solver particle positions, e.g. ClothingSimulationSolver.GetParticleXs().
	* @param ReferencePositions The reference particle positions, e.g. ClothingSimulationSolver.GetAnimationPositions(ParticleRangeId) if using the animation pose as reference.
	* @param Threshold edge ratio threshold to trigger detection.
	*/
	CHAOS_API TArray<TVec2<int32>> GetExtremelyDeformedEdges(TConstArrayView<Softs::FSolverVec3> Positions, const Softs::FSolverVec3* const RefencePositions) const;

private:
	template<int32 Valence, typename Predicate UE_REQUIRES(Valence >= 2 && Valence <= 4)>
	static TArray<TVector<int32, 2>> TrimConstraints(const TArray<TVector<int32, Valence>>& InConstraints, Predicate TrimPredicate)
	{
		TSet<TVec2<int32>> TrimmedConstraints;
		TrimmedConstraints.Reserve(Valence == 2 ? InConstraints.Num() : InConstraints.Num() * Chaos::Utilities::NChooseR(Valence, 2));

		for (const TVector<int32, Valence>& ConstraintV : InConstraints)
		{
			for (int32 i = 0; i < Valence - 1; ++i)
			{
				for (int32 j = i + 1; j < Valence; ++j)
				{
					const int32 IndexI = ConstraintV[i];
					const int32 IndexJ = ConstraintV[j];

					if (!TrimPredicate(IndexI, IndexJ))
					{
						TrimmedConstraints.Add(IndexI <= IndexJ ? TVec2<int32>(IndexI, IndexJ) : TVec2<int32>(IndexJ, IndexI));
					}
				}
			}
		}
		return TrimmedConstraints.Array();
	}

	const TSet<int32>* VertexSet;
	TArray<TVec2<int32>> Constraints;
	const int32 ParticleOffset;
	const int32 ParticleCount;
	FSolverReal ExtremeDeformationThreshold;
	TArray<FSolverReal> Dists;

	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(ExtremeDeformationVertexSelection, bool);  // Selection set name string property, the bool value is not actually used
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(ExtremeDeformationEdgeRatioThreshold, float);
};

}  // End namespace Chaos::Softs
