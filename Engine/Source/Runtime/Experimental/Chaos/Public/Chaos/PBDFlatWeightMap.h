// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/PBDSoftsEvolutionFwd.h"
#include "Chaos/Vector.h"
#include "Containers/ArrayView.h"

namespace Chaos::Softs
{
/**
 * Simple weight map wrapping existing weight values. Also holds low/high values, but these are multiplied against the map each time the operator[] is called.
 */
class FPBDFlatWeightMapView final
{
public:

	FPBDFlatWeightMapView(
		const FSolverVec2& InWeightedValue,
		const TConstArrayView<FSolverReal>& Multipliers,
		int32 InNumElements)
		: NumElements(InNumElements)
		, MapValues(Multipliers.Num() == NumElements ? Multipliers : TConstArrayView<FSolverReal>())
		, OffsetRange(InWeightedValue[0], InWeightedValue[1] - InWeightedValue[0])
	{}

	~FPBDFlatWeightMapView() = default;

	FPBDFlatWeightMapView(const FPBDFlatWeightMapView&) = default;
	FPBDFlatWeightMapView(FPBDFlatWeightMapView&&) = default;
	FPBDFlatWeightMapView& operator=(const FPBDFlatWeightMapView&) = default;
	FPBDFlatWeightMapView& operator=(FPBDFlatWeightMapView&&) = default;

	/** Return the number of values stored in the weight map. */
	int32 Num() const { return MapValues.Num(); }

	/** Return whether this object contains weight map values. */
	bool HasWeightMap() const { return MapValues.Num() > 0 && FMath::Abs(OffsetRange[1]) > UE_KINDA_SMALL_NUMBER; }

	/**
	 * Set the low and high values of the weight map.
	 */
	void SetWeightedValue(const FSolverVec2& InWeightedValue) { OffsetRange[0] = InWeightedValue[0]; OffsetRange[1] = InWeightedValue[1] - InWeightedValue[0]; }

	/**
	 * Return the values set for this map as an Offset and Range (Low = OffsetRange[0], High = OffsetRange[0] + OffsetRange[1])
	 */
	const FSolverVec2& GetOffsetRange() const { return OffsetRange; }

	/**
	 * Lookup for the weighted value at the specified weight map index.
	 * This function will assert if it is called with a non zero index on an empty weight map.
	*/
	FSolverReal operator[](int32 Index) const { return OffsetRange[0] + OffsetRange[1] * MapValues[Index]; }

	/** Return the value at the Low weight. */
	FSolverReal GetLow() const { return OffsetRange[0]; }

	/** Return the value at the High weight. */
	FSolverReal GetHigh() const { return OffsetRange[0] + OffsetRange[1]; }

	FSolverReal GetValue(int32 Index) const
	{
		if (HasWeightMap())
		{
			return (*this)[Index];
		}
		return GetLow();
	}

	/** Return the value when the weight map is not used. */
	explicit operator FSolverReal() const { return GetLow(); }

	void SetMapValues(const TConstArrayView<FSolverReal>& InMapValues)
	{
		if (InMapValues.Num() == NumElements)
		{
			MapValues = InMapValues;
		}
		else
		{
			MapValues = TConstArrayView<FSolverReal>();
		}
	}

	/** Return the table of stiffnesses as a read only array. */
	TConstArrayView<FSolverReal> GetMapValues() const { return MapValues; }

private:
	int32 NumElements;
	TConstArrayView<FSolverReal> MapValues;
	FSolverVec2 OffsetRange;
};

/**
 * Simple weight map just converting from per particle to per constraint values. Also holds low/high values, but these are multiplied against the map each time the operator[] is called.
 */
class FPBDFlatWeightMap final
{
public:

	/**
	 * Weightmap particle constructor. 
	 */
	inline explicit FPBDFlatWeightMap(
		const FSolverVec2& InWeightedValue,
		const TConstArrayView<FRealSingle>& Multipliers = TConstArrayView<FRealSingle>(),
		int32 ParticleCount = 0, // A value of 0 also disables the map
		const FSolverVec2& MultiplierClampRange = FSolverVec2(0., 1.));

	inline FPBDFlatWeightMap(
		const FSolverVec2& InWeightedValue,
		TArray<FRealSingle>&& Multipliers);

	/**
	 * Weightmap constraint constructor. 
	 */
	template<int32 Valence>
	inline explicit FPBDFlatWeightMap(
		const FSolverVec2& InWeightedValue,
		const TConstArrayView<FRealSingle>& Multipliers = TConstArrayView<FRealSingle>(),
		const TConstArrayView<TVector<int32, Valence>>& Constraints = TConstArrayView<TVector<int32, Valence>>(),
		int32 ParticleOffset = INDEX_NONE, // Constraints have usually a particle offset added to them compared to the weight maps that always starts at index 0
		int32 ParticleCount = 0,  // A value of 0 also disables the map
		typename TEnableIf<Valence >= 2 && Valence <= 4>::Type* = nullptr);  // Prevents incorrect valence, the value is actually unused

	~FPBDFlatWeightMap() = default;

	FPBDFlatWeightMap(const FPBDFlatWeightMap&) = default;
	FPBDFlatWeightMap(FPBDFlatWeightMap&&) = default;
	FPBDFlatWeightMap& operator=(const FPBDFlatWeightMap&) = default;
	FPBDFlatWeightMap& operator=(FPBDFlatWeightMap&&) = default;

	/** Return the number of values stored in the weight map. */
	int32 Num() const { return MapView.Num(); }

	/** Return whether this object contains weight map values. */
	bool HasWeightMap() const { return MapView.HasWeightMap(); }

	/**
	 * Set the low and high values of the weight map.
	 */
	void SetWeightedValue(const FSolverVec2& InWeightedValue) { MapView.SetWeightedValue(InWeightedValue); }

	/**
	 * Return the values set for this map as an Offset and Range (Low = OffsetRange[0], High = OffsetRange[0] + OffsetRange[1])
	 */
	const FSolverVec2& GetOffsetRange() const { return MapView.GetOffsetRange(); }

	/**
	 * Lookup for the weighted value at the specified weight map index.
	 * This function will assert if it is called with a non zero index on an empty weight map.
	*/
	FSolverReal operator[](int32 Index) const { return MapView[Index]; }

	/** Return the value at the Low weight. */
	FSolverReal GetLow() const { return MapView.GetLow(); }

	/** Return the value at the High weight. */
	FSolverReal GetHigh() const { return MapView.GetHigh(); }

	/** Return the value when the weight map is not used. */
	explicit operator FSolverReal() const { return (FSolverReal)MapView; }

	/** Return the table of stiffnesses as a read only array. */
	TConstArrayView<FSolverReal> GetMapValues() const { return MapView.GetMapValues(); }

	FSolverReal GetValue(int32 Index) const { return MapView.GetValue(Index); }

	/** Reorder Indices based on Constraint reordering. */
	inline void ReorderIndicesAndShrink(const TArray<int32>& OrigToReorderedIndices, int32 NewNum);
	void ReorderIndices(const TArray<int32>& OrigToReorderedIndices)
	{
		return ReorderIndicesAndShrink(OrigToReorderedIndices, MapValues.Num());
	}

private:
	static TArray<FSolverReal> CalculateMapValues(const TConstArrayView<FRealSingle>& Multipliers, int32 ParticleCount, const FSolverVec2& MultiplierClampRange)
	{
		TArray<FSolverReal> Result;
		if (Multipliers.Num() == ParticleCount && ParticleCount > 0)
		{
			// Copy multipliers with clamp
			Result.SetNumUninitialized(ParticleCount);
			for (int32 Index = 0; Index < ParticleCount; ++Index)
			{
				Result[Index] = FMath::Clamp(Multipliers[Index], MultiplierClampRange[0], MultiplierClampRange[1]);
			}
		}
		return Result;
	}

	template<int32 Valence>
	static TArray<FSolverReal> CalculateMapValues(
		const TConstArrayView<FRealSingle>& Multipliers,
		const TConstArrayView<TVector<int32, Valence>>& Constraints,
		int32 ParticleOffset,
		int32 ParticleCount)
	{
		TArray<FSolverReal> Result;
		const int32 ConstraintCount = Constraints.Num();
		if (Multipliers.Num() == ParticleCount && ParticleCount > 0 && ConstraintCount > 0)
		{
			// Average vertex multipliers to constraints.
			Result.SetNumUninitialized(ConstraintCount);

			for (int32 ConstraintIndex = 0; ConstraintIndex < ConstraintCount; ++ConstraintIndex)
			{
				const TVector<int32, Valence>& Constraint = Constraints[ConstraintIndex];

				FSolverReal Weight = 0.f;
				for (int32 Index = 0; Index < Valence; ++Index)
				{
					Weight += FMath::Clamp((FSolverReal)Multipliers[Constraint[Index] - ParticleOffset], (FSolverReal)0.f, (FSolverReal)1.f);
				}
				Weight /= (FSolverReal)Valence;

				Result[ConstraintIndex] = Weight;
			}
		}
		return Result;
	}

	TArray<FSolverReal> MapValues; 
	FPBDFlatWeightMapView MapView;
};

inline FPBDFlatWeightMap::FPBDFlatWeightMap(
	const FSolverVec2& InWeightedValue,
	const TConstArrayView<FRealSingle>& Multipliers,
	int32 ParticleCount,
	const FSolverVec2& MultiplierClampRange)
	: MapValues(CalculateMapValues(Multipliers, ParticleCount, MultiplierClampRange))
	, MapView(InWeightedValue, TConstArrayView<FSolverReal>(MapValues), ParticleCount)
{
}

inline FPBDFlatWeightMap::FPBDFlatWeightMap(
	const FSolverVec2& InWeightedValue,
	TArray<FRealSingle>&& Multipliers)
	: MapValues(MoveTemp(Multipliers))
	, MapView(InWeightedValue, MapValues, MapValues.Num())
{}

template<int32 Valence>
inline FPBDFlatWeightMap::FPBDFlatWeightMap(
	const FSolverVec2& InWeightedValue,
	const TConstArrayView<FRealSingle>& Multipliers,
	const TConstArrayView<TVector<int32, Valence>>& Constraints,
	int32 ParticleOffset,
	int32 ParticleCount,
	typename TEnableIf<Valence >= 2 && Valence <= 4>::Type*)
	: MapValues(CalculateMapValues(Multipliers, Constraints, ParticleOffset, ParticleCount))
	, MapView(InWeightedValue, TConstArrayView<FSolverReal>(MapValues), Constraints.Num())
{
}

inline void FPBDFlatWeightMap::ReorderIndicesAndShrink(const TArray<int32>& OrigToReorderedConstraintIndices, int32 NewNum)
{
	if (MapValues.Num() == OrigToReorderedConstraintIndices.Num())
	{
		TArray<FSolverReal> ReorderedValues;
		ReorderedValues.SetNumUninitialized(NewNum);
		for (int32 OrigConstraintIndex = 0; OrigConstraintIndex < OrigToReorderedConstraintIndices.Num(); ++OrigConstraintIndex)
		{
			const int32 ReorderedConstraintIndex = OrigToReorderedConstraintIndices[OrigConstraintIndex];
			if (ReorderedConstraintIndex != INDEX_NONE)
			{
				ReorderedValues[ReorderedConstraintIndex] = MapValues[OrigConstraintIndex];
			}
		}
		MapValues = MoveTemp(ReorderedValues);
		MapView.SetMapValues(TConstArrayView<FSolverReal>(MapValues));
	}
	else
	{
		check(MapValues.Num() == 0);
	}
}

}  // End namespace Chaos::Softs
