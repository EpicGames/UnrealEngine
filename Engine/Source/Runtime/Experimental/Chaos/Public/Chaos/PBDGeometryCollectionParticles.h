// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/ArrayCollectionArray.h"
#include "Chaos/PBDRigidClusteredParticles.h"

namespace Chaos
{
	template<class T, int d>
	class TPBDGeometryCollectionParticles : public TPBDRigidClusteredParticles<T, d>
	{
	public:
		TPBDGeometryCollectionParticles()
		    : TPBDRigidClusteredParticles<T, d>()
		{
			InitHelper();
		}
		TPBDGeometryCollectionParticles(const TPBDRigidClusteredParticles<T, d>& Other) = delete;

		TPBDGeometryCollectionParticles(TPBDRigidClusteredParticles<T, d>&& Other)
		    : TPBDRigidClusteredParticles<T, d>(MoveTemp(Other))
		{
			InitHelper();
		}
		~TPBDGeometryCollectionParticles() = default;

		typedef TPBDGeometryCollectionParticleHandle<T, d> THandleType;
		const THandleType* Handle(int32 Index) const { return static_cast<const THandleType*>(TGeometryParticles<T, d>::Handle(Index)); }
		THandleType* Handle(int32 Index) { return static_cast<THandleType*>(TGeometryParticles<T, d>::Handle(Index)); }

	private:
		void InitHelper() { this->MParticleType = EParticleType::GeometryCollection; }
	};
} // namespace Chaos
