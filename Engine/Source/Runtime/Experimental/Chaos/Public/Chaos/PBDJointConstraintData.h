// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Chaos/PBDJointConstraintTypes.h"
#include "Chaos/ParticleDirtyFlags.h"
#include "Chaos/Framework/PhysicsProxyBase.h"
#include "Chaos/Framework/PhysicsSolverBase.h"
#include "Chaos/PBDConstraintBaseData.h"
#include "PhysicsProxy/SingleParticlePhysicsProxyFwd.h"


namespace Chaos
{

	enum class EJointConstraintFlags : uint64_t
	{
		JointTransforms                     = static_cast<uint64_t>(1) << 0,
		CollisionEnabled                    = static_cast<uint64_t>(1) << 1,
		Projection                          = static_cast<uint64_t>(1) << 2,
		ParentInvMassScale                  = static_cast<uint64_t>(1) << 3,
		LinearBreakForce                    = static_cast<uint64_t>(1) << 4,
		AngularBreakTorque                  = static_cast<uint64_t>(1) << 5,
		UserData                            = static_cast<uint64_t>(1) << 6,
		LinearDrive                         = static_cast<uint64_t>(1) << 7,
		AngularDrive                        = static_cast<uint64_t>(1) << 8,
		Stiffness                           = static_cast<uint64_t>(1) << 9,
		Limits                              = static_cast<uint64_t>(1) << 10,
		LinearViolationCallbackThreshold    = static_cast<uint64_t>(1) << 11,
		AngularViolationCallbackThreshold   = static_cast<uint64_t>(1) << 12,

		DummyFlag
	};


	using FJointConstraintDirtyFlags = TDirtyFlags<EJointConstraintFlags>;

	class FJointConstraint : public FConstraintBase
	{
	public:
		using Base = FConstraintBase;

		friend class FPBDRigidsSolver; // friend so we can call ReleaseKinematicEndPoint when unregistering joint.

		CHAOS_API FJointConstraint();
		virtual ~FJointConstraint() override = default;

		const FPBDJointSettings& GetJointSettings()const { return JointSettings.Read(); }
	
		void SetJointSettings(const FPBDJointSettings& NewSettings);

		// If we created particle to serve as kinematic endpoint, track so we can release later. This will add particle to solver.
		CHAOS_API void SetKinematicEndPoint(FSingleParticlePhysicsProxy* InParticle, FPBDRigidsSolver* Solver);

		CHAOS_API FSingleParticlePhysicsProxy* GetKinematicEndPoint() const;

		//See JointProperties for API
		//Each CHAOS_INNER_JOINT_PROPERTY entry will have a Get* and Set*
#define CHAOS_INNER_JOINT_PROPERTY(OuterProp, FuncName, Inner, InnerType) CONSTRAINT_JOINT_PROPERPETY_IMPL2(InnerType, FuncName, OuterProp, Inner)
#include "Chaos/JointProperties.inl"

		CHAOS_API void SetLinearPositionDriveEnabled(TVector<bool, 3> Enabled);

		CHAOS_API void SetLinearVelocityDriveEnabled(TVector<bool, 3> Enabled);

		struct FOutputData
		{
			// Output properties
			bool bIsBreaking = false;
			bool bIsBroken = false;
			bool bDriveTargetChanged = false;
			FVector Force = FVector(0);
			FVector Torque = FVector(0);
			bool bIsViolating = false;
			float LinearViolation = 0.f;
			float AngularViolation = 0.f;
		};
		FOutputData& GetOutputData() { return Output; }

		virtual void SyncRemoteDataImp(FDirtyPropertiesManager& Manager, int32 DataIdx, FDirtyChaosProperties& RemoteData) override
		{
			JointProxies.SyncRemote(Manager, DataIdx, RemoteData);
			JointBodies.SyncRemote(Manager, DataIdx, RemoteData);
			JointSettings.SyncRemote(Manager, DataIdx, RemoteData);
		}

		CHAOS_API void SetParticleProxies(const FProxyBasePair& InJointParticles);
		CHAOS_API void SetPhysicsBodies(const FPhysicsObjectPair& InBodies);

		const FProxyBasePair& GetParticleProxies() const { return  JointProxies.Read().ParticleProxies; }
		const FPhysicsObjectPair& GetPhysicsBodies() const { return  JointBodies.Read().PhysicsBodies; }

	protected:

		CHAOS_API void ReleaseKinematicEndPoint(FPBDRigidsSolver* Solver);
		
		TChaosProperty<FProxyBasePairProperty, EChaosProperty::JointParticleProxies> JointProxies;
		TChaosProperty<FPhysicsObjectPairProperty, EChaosProperty::JointPhysicsObjects> JointBodies;
		TChaosProperty<FPBDJointSettings, EChaosProperty::JointSettings> JointSettings;


		FOutputData Output;

	private:
		// TODO: When we build constraint with only one actor, we spawn this particle to serve as kinematic endpoint
		// to attach to, as Chaos requires two particles currently. This tracks particle that will need to be released with joint.
		FSingleParticlePhysicsProxy* KinematicEndPoint;
	};

} // Chaos



