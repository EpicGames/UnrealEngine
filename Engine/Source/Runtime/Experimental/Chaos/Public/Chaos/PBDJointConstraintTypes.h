// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "Chaos/Core.h"
#include "Chaos/Transform.h"
#include "Chaos/Vector.h"

namespace Chaos
{
	class FPBDJointConstraints;

	class FPBDJointConstraintHandle;

	using FJointPreApplyCallback = TFunction<void(const FReal Dt, const TArray<FPBDJointConstraintHandle*>& InConstraintHandles)>;

	using FJointPostApplyCallback = TFunction<void(const FReal Dt, const TArray<FPBDJointConstraintHandle*>& InConstraintHandles)>;

	using FJointBreakCallback = TFunction<void(FPBDJointConstraintHandle * ConstraintHandle)>;

	enum class EJointMotionType : int32
	{
		Free,
		Limited,
		Locked,
	};

	enum class EJointForceMode : int32
	{
		Acceleration,
		Force,
	};

	enum class EPlasticityType : int32
	{
		Free,
		Shrink,
		Grow,
	};


	/**
	 * The order of the angular constraints (for settings held in vectors etc)
	 */
	// @todo(ccaulfield): rename EJointAngularConstraintType
	enum class EJointAngularConstraintIndex : int32
	{
		Twist,
		Swing2,
		Swing1,
	};

	enum class EJointCylindricalPositionConstraintType : int32
	{
		Axial,
		Radial
	};

	struct FJointConstants
	{
		/** The constraint-space twist axis (X Axis) */
		static inline const FVec3 TwistAxis() { return FVec3(1, 0, 0); }

		/** The constraint-space Swing1 axis (Z Axis) */
		static inline const FVec3 Swing1Axis() { return FVec3(0, 0, 1); }

		/** The constraint-space Swing2 axis (Y Axis) */
		static inline const FVec3 Swing2Axis() { return FVec3(0, 1, 0); }

		/** Get the local-space axis for the specified constraint type. This will be one of the cardinal axes. */
		static inline const FVec3 Axis(const EJointAngularConstraintIndex ConstraintIndex)
		{
			switch (ConstraintIndex)
			{
			case EJointAngularConstraintIndex::Twist:
				return TwistAxis();
			case EJointAngularConstraintIndex::Swing1:
				return Swing1Axis();
			case EJointAngularConstraintIndex::Swing2:
				return Swing2Axis();
			}
		}

		static inline const FVec3 SwingAxis(const EJointAngularConstraintIndex ConstraintIndex)
		{
			return (ConstraintIndex == EJointAngularConstraintIndex::Swing1) ? Swing1Axis() : Swing2Axis();
		}

		static inline const FVec3 OtherSwingAxis(const EJointAngularConstraintIndex ConstraintIndex)
		{
			return (ConstraintIndex == EJointAngularConstraintIndex::Swing1) ? Swing2Axis() : Swing1Axis();
		}

		/** Get the local-space axis index for the specified constraint type. This can be used to index the vectors of a transform matrix for example. */
		static inline const int32 AxisIndex(const EJointAngularConstraintIndex ConstraintIndex)
		{
			if (ConstraintIndex == EJointAngularConstraintIndex::Twist)
			{
				return 0;	// X
			}
			else if (ConstraintIndex == EJointAngularConstraintIndex::Swing1)
			{
				return 2;	// Z
			}
			else
			{
				return 1;	// Y
			}
		}
	};

	class FPBDJointSettings
	{
	public:
		CHAOS_API FPBDJointSettings();

		// Ensure that settings are consistent and within valid ranges. Should be called
		// whenever settings change.
		CHAOS_API void Sanitize();

		// Not currently implemented but needed for templated logic
		bool IsNearlyEqual(const FPBDJointSettings& Other, FReal Leniency = UE_KINDA_SMALL_NUMBER) const
		{
			return *this == Other;
		}

		bool operator==(const FPBDJointSettings& Other) const
		{
			return !FMemory::Memcmp(this, &Other, sizeof(*this));
		}

		// The offsets of the joint frames from the bodies. The ordering is [child, parent]
		FTransformPair ConnectorTransforms;

		FReal Stiffness;
		// The amount of linear projection, between 0 and 1. Only used if bProjectionEnabled is set too.
		FReal LinearProjection;
		// The amount of angular projection, between 0 and 1. Only used if bProjectionEnabled is set too.
		FReal AngularProjection;
		FReal ShockPropagation;
		FReal TeleportDistance;
		FReal TeleportAngle;			// Radians
		// This inverse scales the parent mass (and inertia) during the solve. By setting this to
		// zero, the constraint will treat the parent body as infinitely massive, and thus the
		// constraint will only operate on the child body.
		FReal ParentInvMassScale;

		bool bCollisionEnabled;
		bool bProjectionEnabled;		// @chaos(todo): remove - implied by Linear/AngularProjection and teleport settings
		bool bShockPropagationEnabled;	// @chaos(todo): remove - implied by ShockPropagation
		bool bMassConditioningEnabled;

		// If enabled, then a linear solver will be used. However, note that the immediate solver
		// doesn't support mixing both linear and non-linear constraints, so in that case you should
		// make all constraints be either linear or non-linear.
		bool bUseLinearSolver;

		TVector<EJointMotionType, 3> LinearMotionTypes;
		FReal LinearLimit;

		// Order is twist, swing1, swing2 
		TVector<EJointMotionType, 3> AngularMotionTypes;
		// The values here are in radians
		FVec3 AngularLimits;

		bool bSoftLinearLimitsEnabled;
		bool bSoftTwistLimitsEnabled;
		bool bSoftSwingLimitsEnabled;
		EJointForceMode LinearSoftForceMode;
		EJointForceMode AngularSoftForceMode;
		FReal SoftLinearStiffness;
		FReal SoftLinearDamping;
		FReal SoftTwistStiffness;
		FReal SoftTwistDamping;
		FReal SoftSwingStiffness;
		FReal SoftSwingDamping;

		FReal LinearRestitution;
		FReal TwistRestitution;
		FReal SwingRestitution;

		FReal LinearContactDistance;
		FReal TwistContactDistance;
		FReal SwingContactDistance;

		FVec3 LinearDrivePositionTarget;
		FVec3 LinearDriveVelocityTarget;
		TVector<bool, 3> bLinearPositionDriveEnabled;
		TVector<bool, 3> bLinearVelocityDriveEnabled;
		EJointForceMode LinearDriveForceMode;
		FVec3 LinearDriveStiffness;
		FVec3 LinearDriveDamping;
		FVec3 LinearDriveMaxForce;

		FRotation3 AngularDrivePositionTarget;
		FVec3 AngularDriveVelocityTarget;

		bool bAngularSLerpPositionDriveEnabled;
		bool bAngularSLerpVelocityDriveEnabled;
		bool bAngularTwistPositionDriveEnabled;
		bool bAngularTwistVelocityDriveEnabled;
		bool bAngularSwingPositionDriveEnabled;
		bool bAngularSwingVelocityDriveEnabled;
		EJointForceMode AngularDriveForceMode;
		FVec3 AngularDriveStiffness;
		FVec3 AngularDriveDamping;
		FVec3 AngularDriveMaxTorque;

		FReal LinearBreakForce;
		FReal LinearViolationCallbackThreshold;
		FReal LinearPlasticityLimit;
		EPlasticityType LinearPlasticityType;
		FReal LinearPlasticityInitialDistanceSquared;
		FReal AngularBreakTorque;
		FReal AngularViolationCallbackThreshold;
		FReal AngularPlasticityLimit;

		FReal ContactTransferScale;

		void* UserData;
	};

	class FPBDJointSolverSettings
	{
	public:
		CHAOS_API FPBDJointSolverSettings();

		CHAOS_API FPBDJointSolverSettings(const FPBDJointSolverSettings& InSettings);

		CHAOS_API FPBDJointSolverSettings& operator=(const FPBDJointSolverSettings& Other);

		// Tolerances
		FReal SwingTwistAngleTolerance;
		FReal PositionTolerance;
		FReal AngleTolerance;

		// Stability control
		FReal MinParentMassRatio;
		FReal MaxInertiaRatio;

		// Solver Stiffness (increases over iterations)
		FReal MinSolverStiffness;
		FReal MaxSolverStiffness;
		int32 NumIterationsAtMaxSolverStiffness;
		int32 NumShockPropagationIterations;

		// Whether to use the linear or non-linear joint solver
		UE_DEPRECATED(5.5, "No longer used, see per-constraint flag in FPBDJointSettings::bUseLinearSolver")
		bool bUseLinearSolver;

		// Whether use vectorization in single precision, only available on the linear solver
		bool bUseSimd;

		// Whether the joints need to be sorted (only required for RBAN - the world solver uses the constraint graph for ordering)
		bool bSortEnabled;

		// Whether to solve rotation then position limits (true), or vice versa
		// Solving position last leads to less separation at the joints when limits are being forced
		bool bSolvePositionLast;

		// Whether joints are position-based or velocity-based in the solver
		bool bUsePositionBasedDrives;

		// @todo(chaos): remove these TEMP overrides for testing
		bool bEnableTwistLimits;
		bool bEnableSwingLimits;
		bool bEnableDrives;
		FReal LinearStiffnessOverride;
		FReal TwistStiffnessOverride;
		FReal SwingStiffnessOverride;
		FReal LinearProjectionOverride;
		FReal AngularProjectionOverride;
		FReal ShockPropagationOverride;
		FReal LinearDriveStiffnessOverride;
		FReal LinearDriveDampingOverride;
		FReal AngularDriveStiffnessOverride;
		FReal AngularDriveDampingOverride;
		FReal SoftLinearStiffnessOverride;
		FReal SoftLinearDampingOverride;
		FReal SoftTwistStiffnessOverride;
		FReal SoftTwistDampingOverride;
		FReal SoftSwingStiffnessOverride;
		FReal SoftSwingDampingOverride;
	};

}
