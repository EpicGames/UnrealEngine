// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "Chaos/ParticleHandleFwd.h"
#include "Chaos/PBDJointConstraintTypes.h"
#include "Chaos/Collision/CollisionApplyType.h"


namespace Chaos
{
	class FPBDJointUtilities
	{
	public:
		static CHAOS_API void GetSphericalAxisDelta(
			const FVec3& X0,
			const FVec3& X1,
			FVec3& Axis,
			FReal& Delta);

		static CHAOS_API void GetCylindricalAxesDeltas(
			const FRotation3& R0,
			const FVec3& X0,
			const FVec3& X1,
			const int32 CylinderAxisIndex,
			FVec3& CylinderAxis,
			FReal& CylinderDelta,
			FVec3& RadialAxis,
			FReal& RadialDelta);

		static CHAOS_API void GetPlanarAxisDelta(
			const FRotation3& R0,
			const FVec3& X0,
			const FVec3& X1,
			const int32 PlaneAxisIndex,
			FVec3& Axis,
			FReal& Delta);

		static CHAOS_API void DecomposeSwingTwistLocal(
			const FRotation3& R0, 
			const FRotation3& R1, 
			FRotation3& R01Swing, 
			FRotation3& R01Twist);
		
		static void CHAOS_API GetSwingTwistAngles(
			const FRotation3& R0, 
			const FRotation3& R1, 
			FReal& TwistAngle, 
			FReal& Swing1Angle, 
			FReal& Swing2Angle);

		static CHAOS_API FReal GetTwistAngle(
			const FRotation3& InTwist);
		
		static CHAOS_API void GetTwistAxisAngle(
			const FRotation3& R0,
			const FRotation3& R1,
			FVec3& Axis,
			FReal& Angle);

		static CHAOS_API void GetConeAxisAngleLocal(
			const FRotation3& R0,
			const FRotation3& R1,
			const FReal AngleTolerance,
			FVec3& AxisLocal,
			FReal& Angle);

		static CHAOS_API void GetCircularConeAxisErrorLocal(
			const FRotation3& R0,
			const FRotation3& R1,
			const FReal SwingLimit,
			FVec3& AxisLocal,
			FReal& Error);

		static CHAOS_API void GetEllipticalConeAxisErrorLocal(
			const FRotation3& R0,
			const FRotation3& R1,
			const FReal SwingLimitY,
			const FReal SwingLimitZ,
			FVec3& AxisLocal,
			FReal& Error);

		static CHAOS_API void GetLockedSwingAxisAngle(
			const FRotation3& R0,
			const FRotation3& R1,
			const EJointAngularConstraintIndex SwingConstraintIndex,
			FVec3& Axis,
			FReal& Angle);

		static CHAOS_API void GetDualConeSwingAxisAngle(
			const FRotation3& R0,
			const FRotation3& R1,
			const EJointAngularConstraintIndex SwingConstraintIndex,
			FVec3& Axis,
			FReal& Angle);

		static CHAOS_API void GetSwingAxisAngle(
			const FRotation3& R0,
			const FRotation3& R1,
			const FReal AngleTolerance,
			const EJointAngularConstraintIndex SwingConstraintIndex,
			FVec3& Axis,
			FReal& Angle);

		static CHAOS_API void GetLockedRotationAxes(
			const FRotation3& R0, 
			const FRotation3& R1, 
			FVec3& Axis0, 
			FVec3& Axis1, 
			FVec3& Axis2);

		static CHAOS_API FReal GetConeAngleLimit(
			const FPBDJointSettings& JointSettings,
			const FVec3& SwingAxisLocal,
			const FReal SwingAngle);

		/**
		 * Increase the lower inertia components to ensure that the maximum ratio between any pair of elements is MaxRatio.
		 *
		 * @param InI The input inertia.
		 * @return An altered inertia so that the minimum element is at least MaxElement/MaxRatio.
		 */
		static CHAOS_API FVec3 ConditionInertia(
			const FVec3& InI, 
			const FReal MaxRatio);

		/**
		 * Increase the IParent inertia so that its largest component is at least MinRatio times the largest IChild component.
		 * This is used to condition joint chains for more robust solving with low iteration counts or larger time steps.
		 *
		 * @param IParent The input inertia.
		 * @param IChild The input inertia.
		 * @param OutIParent The output inertia.
		 * @param MinRatio Parent inertia will be at least this multiple of child inertia
		 * @return The max/min ratio of InI elements.
		 */
		static CHAOS_API FVec3 ConditionParentInertia(
			const FVec3& IParent, 
			const FVec3& IChild, 
			const FReal MinRatio);

		static CHAOS_API FReal ConditionParentMass(
			const FReal MParent, 
			const FReal MChild, 
			const FReal MinRatio);

		static CHAOS_API void ConditionInverseMassAndInertia(
			const FReal& InInvMParent,
			const FReal& InInvMChild,
			const FVec3& InInvIParent,
			const FVec3& InInvIChild,
			const FReal MinParentMassRatio,
			const FReal MaxInertiaRatio,
			FReal& OutInvMParent,
			FReal& OutInvMChild,
			FVec3& OutInvIParent,
			FVec3& OutInvIChild);

		static CHAOS_API void ConditionInverseMassAndInertia(
			FReal& InOutInvMParent,
			FReal& InOutInvMChild,
			FVec3& InOutInvIParent,
			FVec3& InOutInvIChild,
			const FReal MinParentMassRatio,
			const FReal MaxInertiaRatio);

		static bool GetSoftLinearLimitEnabled(
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		static bool GetSoftTwistLimitEnabled(
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		static bool GetSoftSwingLimitEnabled(
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		static FReal GetLinearStiffness(
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		static FReal GetSoftLinearStiffness(
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		static FReal GetSoftLinearDamping(
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		static FReal GetTwistStiffness(
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		static FReal GetSoftTwistStiffness(
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		static FReal GetSoftTwistDamping(
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		static FReal GetSwingStiffness(
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		static FReal GetSoftSwingStiffness(
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		static FReal GetSoftSwingDamping(
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		static FReal GetLinearDriveStiffness(
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings,
			const int32 AxisIndex);

		static FReal GetLinearDriveDamping(
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings,
			const int32 AxisIndex);

		static FReal GetAngularTwistDriveStiffness(
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		static FReal GetAngularTwistDriveDamping(
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		static FReal GetAngularTwistDriveTorqueLimit(
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		static FReal GetAngularSwingDriveStiffness(
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		static FReal GetAngularSwingDriveDamping(
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		static FReal GetAngularSwingDriveTorqueLimit(
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		static FReal GetAngularSLerpDriveStiffness(
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		static FReal GetAngularSLerpDriveDamping(
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		static FReal GetLinearProjection(
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		static FReal GetAngularProjection(
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		static bool GetLinearSoftAccelerationMode(
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		static bool GetAngularSoftAccelerationMode(
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		static bool GetLinearDriveAccelerationMode(
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		static bool GetAngularDriveAccelerationMode(
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		static FReal GetShockPropagationInvMassScale(
			const FPBDJointSolverSettings& SolverSettings,
			const FPBDJointSettings& JointSettings);

		static FVec3 GetSphereLimitedPositionError(const FVec3& CX, const FReal Radius);
		static FVec3 GetCylinderLimitedPositionError(const FVec3& CX, const FVec3& Axis, const FReal Limit, const EJointMotionType AxisMotion);
		static FVec3 GetLineLimitedPositionError(const FVec3& CX, const FVec3& Axis, const FReal Limit, const EJointMotionType AxisMotion);
		static FVec3 GetLimitedPositionError(const FPBDJointSettings& JointSettings, const FRotation3& R0, const FVec3& CX);
		static CHAOS_API bool GetLinearIsFree(const FPBDJointSettings& JointSettings);
	};
}

