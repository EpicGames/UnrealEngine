// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"

#include "Chaos/ConstraintHandle.h"
#include "Chaos/Collision/CollisionApplyType.h"
#include "Chaos/Evolution/IndexedConstraintContainer.h"
#include "Chaos/Island/IslandManagerFwd.h"
#include "Chaos/ParticleHandleFwd.h"
#include "Chaos/PBDJointConstraintTypes.h"
#include "Chaos/PBDJointConstraintData.h"
#include "Chaos/GeometryParticles.h"

namespace Chaos
{

namespace Private
	{
		template <typename JointSolverType>
		class TPBDJointContainerSolver;
	}

	/**
	 * A handle to a joint constraint held in a joint container (FPBDJointConstraints) by index.
	*/
	class FPBDJointConstraintHandle final : public TIndexedContainerConstraintHandle<FPBDJointConstraints>
	{
	public:
		using Base = TIndexedContainerConstraintHandle<FPBDJointConstraints>;
		using FConstraintContainer = FPBDJointConstraints;

		FPBDJointConstraintHandle();
		FPBDJointConstraintHandle(FConstraintContainer* InConstraintContainer, int32 InConstraintIndex);

		CHAOS_API void SetConstraintEnabled(bool bEnabled);

		CHAOS_API void CalculateConstraintSpace(FVec3& OutXa, FMatrix33& OutRa, FVec3& OutXb, FMatrix33& OutRb) const;

		

		CHAOS_API bool IsConstraintEnabled() const;
		CHAOS_API bool IsConstraintBroken() const;
		CHAOS_API bool IsConstraintBreaking() const;
		CHAOS_API void ClearConstraintBreaking();
		CHAOS_API bool IsConstraintViolating() const;
		CHAOS_API void ClearConstraintViolating();
		CHAOS_API bool IsDriveTargetChanged() const;
		CHAOS_API void ClearDriveTargetChanged();
		CHAOS_API FVec3 GetLinearImpulse() const;
		CHAOS_API FVec3 GetAngularImpulse() const;
		CHAOS_API float GetLinearViolation() const;
		CHAOS_API float GetAngularViolation() const;

		CHAOS_API const FPBDJointSettings& GetSettings() const;
		const FPBDJointSettings& GetJointSettings() const { return GetSettings(); }	//needed for property macros
	
		void SetJointState(const class FPBDJointState& NewState);

		// Note that this is the "regular" way to change any settings, but note that if it results
		// in a change of state/mode there may be some overhead. For simple/numerical changes, it
		// may be faster to call one of the specific "Set" functions.
		CHAOS_API void SetSettings(const FPBDJointSettings& Settings);

		// Individual properties can be set if the settings don't need to be Sanitized().

		CHAOS_API void SetParentConnectorLocation(const FVec3 Location);
		CHAOS_API void SetParentConnectorRotation(const FQuat Rotation);
		CHAOS_API void SetChildConnectorLocation(const FVec3 Location);
		CHAOS_API void SetChildConnectorRotation(const FQuat Rotation);

		CHAOS_API void SetLinearDrivePositionTarget(const FVec3 Target);
		CHAOS_API void SetAngularDrivePositionTarget(const FQuat Target);

		CHAOS_API void SetLinearDriveVelocityTarget(const FVec3 Target);
		CHAOS_API void SetAngularDriveVelocityTarget(const FVec3 Target);

		CHAOS_API void SetLinearDriveStiffness(const FVec3 Stiffness);
		CHAOS_API void SetLinearDriveDamping(const FVec3 Damping);
		CHAOS_API void SetLinearDriveMaxForce(const FVec3 MaxForce);
		CHAOS_API void SetLinearDriveForceMode(EJointForceMode ForceMode);

		CHAOS_API void SetAngularDriveStiffness(const FVec3 Stiffness);
		CHAOS_API void SetAngularDriveDamping(const FVec3 Damping);
		CHAOS_API void SetAngularDriveMaxTorque(const FVec3 MaxTorque);
		CHAOS_API void SetAngularDriveForceMode(EJointForceMode ForceMode);

		CHAOS_API void SetCollisionEnabled(const bool bCollisionEnabled);
		CHAOS_API void SetParentInvMassScale(const FReal ParentInvMassScale);

		/**
		 * This allows the most common drive parameters to be set in one call. Note that the 
		 * individual drive elements will be enabled/disabled depending on the strength/damping
		 * values passed in.
		 *
		 * Angular values are passed in as (swing, twist, slerp)
		 */
		CHAOS_API void SetDriveParams(
			const FVec3 LinearStiffness, const FVec3 LinearDamping, const FVec3 MaxForce,
			const FVec3 AngularStiffness, const FVec3 AngularDamping, const FVec3 MaxTorque);

		CHAOS_API TVec2<FGeometryParticleHandle*> GetConstrainedParticles() const override final;

		static const FConstraintHandleTypeID& StaticType()
		{
			static FConstraintHandleTypeID STypeID(TEXT("FJointConstraintHandle"), &FIndexedConstraintHandle::StaticType());
			return STypeID;
		}

		CHAOS_API ESyncState SyncState() const;
		CHAOS_API void SetSyncState(ESyncState SyncState);

		CHAOS_API void SetEnabledDuringResim(bool bEnabled);
		CHAOS_API bool IsEnabledDuringResim() const;
		CHAOS_API EResimType ResimType() const;
		CHAOS_API void SetResimType(EResimType ResimType);

		UE_DEPRECATED(5.2, "No longer used")
		CHAOS_API int32 GetConstraintIsland() const;
		UE_DEPRECATED(5.2, "No longer used")
		CHAOS_API int32 GetConstraintLevel() const;
		UE_DEPRECATED(5.2, "No longer used")
		CHAOS_API int32 GetConstraintColor() const;

		void SetConstraintContainer(FPBDConstraintContainer* NewConstraintContainer) { ConstraintContainer = NewConstraintContainer;  }
		bool IsInContainer(FPBDConstraintContainer* TestContainer) { return TestContainer == ConstraintContainer;  }

	protected:
		using Base::ConstraintIndex;
		using Base::ConcreteContainer;
	private:
		// Our own direct access to the settings, to modify things that don't need to go through sanitize
		FPBDJointSettings& GetConstraintSettingsInternal();

		bool bLinearPlasticityInitialized;
		bool bAngularPlasticityInitialized;
	};

	/**
	 * Peristent variable state for a joint
	*/
	class FPBDJointState
	{
	public:
		CHAOS_API FPBDJointState();

		int32 Island;
		int32 Level;
		int32 Color;
		int32 IslandSize;
		bool bDisabled;
		bool bBroken;
		bool bBreaking;
		bool bViolating;
		bool bDriveTargetChanged;
		FVec3 LinearImpulse;
		FVec3 AngularImpulse;
		float LinearViolation;
		float AngularViolation;
		EResimType ResimType = EResimType::FullResim;
		ESyncState SyncState = ESyncState::InSync;
		bool bEnabledDuringResim = true;
	};

	/**
	 * A set of joint restricting up to 6 degrees of freedom, with linear and angular limits.
	 */
	class FPBDJointConstraints : public FPBDConstraintContainer
	{
	public:
		using Base = FPBDConstraintContainer;

		using FConstraintContainerHandle = FPBDJointConstraintHandle;
		using FConstraintHandleAllocator = TConstraintHandleAllocator<FPBDJointConstraints>;
		using FHandles = TArray<FConstraintContainerHandle*>;

		CHAOS_API FPBDJointConstraints();

		CHAOS_API virtual ~FPBDJointConstraints();

		/**
		 * Get the solver settings (used by TPBDJointContainerSolver)
		*/
		CHAOS_API const FPBDJointSolverSettings& GetSettings() const;

		/*
		* Modify the solver settings
		*/
		CHAOS_API void SetSettings(const FPBDJointSolverSettings& InSettings);

		/**
		* Whether to use a linear or non-linear joint solver. Non-linear is more stable but much more expensive.
		* A linear solver is used by default. This API is deprecated. See FPBDJointSettings::bUseLinearSolver instead. 
		*/
		UE_DEPRECATED(5.5, "No longer used, see per-constraint flag in FPBDJointSettings::bUseLinearSolver")
		void SetUseLinearJointSolver(const bool bInEnable) { }

		/**
		* Whether to use simd in single precision on the linear joint solver. 
		*/
		void SetUseSimd(const bool bInEnable) { Settings.bUseSimd = bInEnable; }

		/**
		 * Whether to sort the joints internally. Sort will be triggered on any tick when a joint was added. Only needed for RBAN.
		*/
		void SetSortEnabled(const bool bInEnable) { Settings.bSortEnabled = bInEnable; }

		/**
		 * Get the number of constraints.
		 */
		CHAOS_API int32 NumConstraints() const;

		/**
		 * Add a constraint with particle-space constraint offsets.
		 * @todo(chaos): clean up this set of functions (now that ConnectorTransforms is in the settings, calling AddConstraint then SetSettings leads 
		 * to unexpected behaviour - overwriting the ConnectorTransforms with Identity)
		 */
		CHAOS_API FConstraintContainerHandle* AddConstraint(const FParticlePair& InConstrainedParticles, const FRigidTransform3& WorldConstraintFrame);
		CHAOS_API FConstraintContainerHandle* AddConstraint(const FParticlePair& InConstrainedParticles, const FTransformPair& ConnectorTransforms);
		CHAOS_API FConstraintContainerHandle* AddConstraint(const FParticlePair& InConstrainedParticles, const FPBDJointSettings& InConstraintSettings);

		void SetConstraintState(int32 ConstraintIndex, const FPBDJointState& InState);

		/**
		 * Remove the specified constraint.
		 */
		CHAOS_API void RemoveConstraint(int ConstraintIndex);
		void RemoveConstraints(const TSet<TGeometryParticleHandle<FReal, 3>*>& RemovedParticles) {}

		/*
		* Disconnect the constraints from the attached input particles. 
		* This will set the constrained Particle elements to nullptr and 
		* set the Enable flag to false.
		* 
		* The constraint is unuseable at this point and pending deletion. 
		*/
		CHAOS_API void DisconnectConstraints(const TSet<TGeometryParticleHandle<FReal, 3>*>& RemovedParticles);

		/*
		 * Whether the constraint is enabled
		 */
		CHAOS_API bool IsConstraintEnabled(int32 ConstraintIndex) const;

		/*
		 * Whether the constraint is broken
		 */
		CHAOS_API bool IsConstraintBroken(int32 ConstraintIndex) const;

		/*
		 * Whether the constraint was broken this frame (transient flag for use by event system)
		 */
		CHAOS_API bool IsConstraintBreaking(int32 ConstraintIndex) const;

		/*
		 * Clear the transient constraint braking state (called by event system when it has used the flag)
		 */
		CHAOS_API void ClearConstraintBreaking(int32 ConstraintIndex);

		/*
		 * Whether the constraint was violated beyond a threshold this frame (transient flag for use by event system)
		 */
		CHAOS_API bool IsConstraintViolating(int32 ConstraintIndex) const;

		/*
		 * Clear the transient constraint violation state (called by event system when it has used the flag)
		 */
		CHAOS_API void ClearConstraintViolating(int32 ConstraintIndex);

		/*
		 * Whether the drive target has changed
		 */
		CHAOS_API bool IsDriveTargetChanged(int32 ConstraintIndex) const;

		/*
		 * Clear the drive target state
		 */
		CHAOS_API void ClearDriveTargetChanged(int32 ConstraintIndex);

		/*
		 * Enable or disable a constraints
		 */
		CHAOS_API void SetConstraintEnabled(int32 ConstraintIndex, bool bEnabled);

		/*
		* Set Drive Target Changed State
		*/
		CHAOS_API void SetDriveTargetChanged(int32 ConstraintIndex, bool bTargetChanged);

		/*
		 * Force a constraints to break
		 */
		CHAOS_API void BreakConstraint(int32 ConstraintIndex);

		/**
		 * Repair a broken constraints (does not adjust particle positions)
		 */
		CHAOS_API void FixConstraint(int32 ConstraintIndex);

		/**
		 * Set the break callback. This will be called once after the constraint solver phase on the \
		 * tick when the constraint is broken. There is only one callback allowed - it is usually a 
		 * method on the owner (e.g., the Evolution object) which will probably dispatch its own event.
		*/
		CHAOS_API void SetBreakCallback(const FJointBreakCallback& Callback);

		/**
		 * Remove the previously assigned break callback.
		*/
		CHAOS_API void ClearBreakCallback();

		/**
		 * All of the constraints in the container, including inactive
		*/
		FHandles& GetConstraintHandles()
		{
			return Handles;
		}

		/**
		 * All of the constraints in the container, including inactive
		*/
		const FHandles& GetConstConstraintHandles() const
		{
			return Handles;
		}

		/**
		 * Get a joint constraint by index
		*/
		CHAOS_API const FConstraintContainerHandle* GetConstraintHandle(int32 ConstraintIndex) const;

		/**
		 * Get a joint constraint by index
		*/
		CHAOS_API FConstraintContainerHandle* GetConstraintHandle(int32 ConstraintIndex);

		/**
		 * Get the particles that are affected by the specified constraint.
		 */
		CHAOS_API const FParticlePair& GetConstrainedParticles(int32 ConstraintIndex) const;

		/**
		 * Get the settings for a joint constraint by index
		*/
		CHAOS_API const FPBDJointSettings& GetConstraintSettings(int32 ConstraintIndex) const;

		/**
		 * Set the settings for a joint constraint by index
		*/
		CHAOS_API void SetConstraintSettings(int32 ConstraintIndex, const FPBDJointSettings& InConstraintSettings);

		/**
		 * Set the linear drive target for a constraint by index
		*/
		CHAOS_API void SetLinearDrivePositionTarget(int32 ConstraintIndex, FVec3 InLinearDrivePositionTarget);

		/**
		 * Set the angular drive target for a constraint by index
		*/
		CHAOS_API void SetAngularDrivePositionTarget(int32 ConstraintIndex, FRotation3 InAngularDrivePositionTarget);

		/**
		 * Set the option to use linear solver for a constraint by index
		*/
		CHAOS_API void SetUseLinearSolver(int32 ConstraintIndex, bool InbUseLinearSolver);

		/**
		* The total linear impulse applied by the constraint
		*/
		CHAOS_API FVec3 GetConstraintLinearImpulse(int32 ConstraintIndex) const;

		/**
		* The total linear angular applied by the constraint
		*/
		CHAOS_API FVec3 GetConstraintAngularImpulse(int32 ConstraintIndex) const;

		CHAOS_API float GetConstraintLinearViolation(int32 ConstraintIndex) const;
		CHAOS_API float GetConstraintAngularViolation(int32 ConstraintIndex) const;

		CHAOS_API ESyncState GetConstraintSyncState(int32 ConstraintIndex) const;
		CHAOS_API void SetConstraintSyncState(int32 ConstraintIndex, ESyncState SyncState);
		
		CHAOS_API void SetConstraintEnabledDuringResim(int32 ConstraintIndex, bool bEnabled);
		
		CHAOS_API bool IsConstraintEnabledDuringResim(int32 ConstraintIndex) const;
		
		CHAOS_API EResimType GetConstraintResimType(int32 ConstraintIndex) const;
		CHAOS_API void SetConstraintResimType(int32 ConstraintIndex, EResimType ResimType);

		//
		// FConstraintContainer Implementation
		//
		CHAOS_API virtual TUniquePtr<FConstraintContainerSolver> CreateSceneSolver(const int32 Priority) override final;
		CHAOS_API virtual TUniquePtr<FConstraintContainerSolver> CreateGroupSolver(const int32 Priority) override final;
		virtual int32 GetNumConstraints() const override final { return NumConstraints(); }
		virtual void ResetConstraints() override final {}
		CHAOS_API virtual void AddConstraintsToGraph(Private::FPBDIslandManager& IslandManager) override final;
		CHAOS_API virtual void PrepareTick() override final;
		CHAOS_API virtual void UnprepareTick() override final;

		// Called by the joint solver at the end of the constraint solver phase
		CHAOS_API void SetSolverResults(const int32 ConstraintIndex, const FVec3& LinearImpulse, const FVec3& AngularImpulse, const float LinearViolation, const float AngularViolation, const bool bIsBroken, const bool bIsViolating, const FSolverBody* SolverBody0, const FSolverBody* SolverBody1);

		// @todo(chaos): only needed for RBAN, and should be private or moved to the solver
		CHAOS_API int32 GetConstraintIsland(int32 ConstraintIndex) const;
		CHAOS_API int32 GetConstraintLevel(int32 ConstraintIndex) const;
		CHAOS_API int32 GetConstraintColor(int32 ConstraintIndex) const;

		// This will force a re-evaluation of the sorting of constraints - needed if objects switch
		// between kinematic/dynamic, for example.
		void MarkJointsDirty() { bJointsDirty = true; }

		// @todo(chaos): functionality no longer supported here (see TPBDJointContainerSolver)
		UE_DEPRECATED(5.2, "Joint Solver API moved to TPBDJointContainerSolver")
		void AddBodies(FSolverBodyContainer& SolverBodyContainer) {}
		UE_DEPRECATED(5.2, "Joint Solver API moved to TPBDJointContainerSolver")
		void GatherInput(const FReal Dt){}
		UE_DEPRECATED(5.2, "Joint Solver API moved to TPBDJointContainerSolver")
		void ScatterOutput(const FReal Dt){}
		UE_DEPRECATED(5.2, "Joint Solver API moved to TPBDJointContainerSolver")
		void ApplyPositionConstraints(const FReal Dt, const int32 It, const int32 NumIts) {}
		UE_DEPRECATED(5.2, "Joint Solver API moved to TPBDJointContainerSolver")
		void ApplyVelocityConstraints(const FReal Dt, const int32 It, const int32 NumIts) {}
		UE_DEPRECATED(5.2, "Joint Solver API moved to TPBDJointContainerSolver")
		void ApplyProjectionConstraints(const FReal Dt, const int32 It, const int32 NumIts) {}
		UE_DEPRECATED(5.2, "Joint Solver API moved to TPBDJointContainerSolver")
		void AddBodies(const TArrayView<int32>& ConstraintIndices, FSolverBodyContainer& SolverBodyContainer) {}
		UE_DEPRECATED(5.2, "Joint Solver API moved to TPBDJointContainerSolver")
		void GatherInput(const TArrayView<int32>& ConstraintIndices, const FReal Dt) {}
		UE_DEPRECATED(5.2, "Joint Solver API moved to TPBDJointContainerSolver")
		void ScatterOutput(const TArrayView<int32>& ConstraintIndices, const FReal Dt) {}
		UE_DEPRECATED(5.2, "Joint Solver API moved to TPBDJointContainerSolver")
		void ApplyPositionConstraints(const TArrayView<int32>& ConstraintIndices, const FReal Dt, const int32 It, const int32 NumIts) {}
		UE_DEPRECATED(5.2, "Joint Solver API moved to TPBDJointContainerSolver")
		void ApplyVelocityConstraints(const TArrayView<int32>& ConstraintIndices, const FReal Dt, const int32 It, const int32 NumIts) {}
		UE_DEPRECATED(5.2, "Joint Solver API moved to TPBDJointContainerSolver")
		void ApplyProjectionConstraints(const TArrayView<int32>& ConstraintIndices, const FReal Dt, const int32 It, const int32 NumIts) {}
		UE_DEPRECATED(5.2, "Joint Solver API moved to TPBDJointContainerSolver")
		void GatherInput(const int32 ConstraintIndex, const FReal Dt) {}
		UE_DEPRECATED(5.2, "Joint Solver API moved to TPBDJointContainerSolver")
		void ScatterOutput(const int32 ConstraintIndex, const FReal Dt) {}

		bool GetUseLinearSolver() { return bUseLinearSolver; }

		void SetUseLinearSolver(const bool bUseLinearSolverIn) { bUseLinearSolver = bUseLinearSolverIn; }

	private:
		friend class FPBDJointConstraintHandle;

		friend struct FPBDJointCombinedConstraints;

		CHAOS_API void GetConstrainedParticleIndices(const int32 ConstraintIndex, int32& Index0, int32& Index1) const;
		CHAOS_API void CalculateConstraintSpace(int32 ConstraintIndex, FVec3& OutX0, FMatrix33& OutR0, FVec3& OutX1, FMatrix33& OutR1) const;
		
		CHAOS_API void ColorConstraints();
		CHAOS_API void SortConstraints();

		CHAOS_API bool ShouldBeInGraph(const int32 ConstraintIndex) const;

		CHAOS_API void ApplyPlasticityLimits(const int32 ConstraintIndex, const FSolverBody& SolverBody0, const FSolverBody& SolverBody1);

		CHAOS_API void SetConstraintBroken(int32 ConstraintIndex, bool bBroken);
		CHAOS_API void SetConstraintBreaking(int32 ConstraintIndex, bool bBreaking);

		FPBDJointSolverSettings Settings;

		TArray<FPBDJointSettings> ConstraintSettings;
		TArray<FParticlePair> ConstraintParticles;
		TArray<FPBDJointState> ConstraintStates;

		FHandles Handles;
		FConstraintHandleAllocator HandleAllocator;
		bool bJointsDirty;

		FJointBreakCallback BreakCallback;

		bool bUseLinearSolver;

	};

	struct FPBDJointCombinedConstraints
	{
		CHAOS_API FPBDJointCombinedConstraints();
		void ConvertLinearConstraintToNonLinear(const int32 LinearConstraintIndex);
		void ConvertNonLinearConstraintToLinear(const int32 NonLinearConstraintIndex);
		CHAOS_API void MergeLinearConstraintsIntoNonLinearConstraints();
		CHAOS_API void MergeNonLinearConstraintsIntoLinearConstraints();
		int32 NumConstraints() { return LinearConstraints.NumConstraints() + NonLinearConstraints.NumConstraints(); }

		FPBDJointConstraints LinearConstraints;
		FPBDJointConstraints NonLinearConstraints;
	};
	

}
