// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/PBDFlatWeightMap.h"
#include "Chaos/PBDSoftsEvolutionFwd.h"
#include "Chaos/SkinnedTriangleMesh.h"
#include "Chaos/TriangleMesh.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "Chaos/ParticlesRange.h"
#include "Chaos/ImplicitFwd.h"

namespace Chaos::Softs
{
	class FPBDKinematicTriangleMeshCollisions
	{
	public:
		static constexpr int32 MaxKinematicConnectionsPerPoint = 3;

		FPBDKinematicTriangleMeshCollisions(int32 InNumParticles,
			const FPBDFlatWeightMap& InThickness,
			const FPBDFlatWeightMap& InFrictionCoefficient,
			const FSolverReal InStiffness,
			const FSolverReal InColliderThickness)
			: NumParticles(InNumParticles)
			, Thickness(InThickness)
			, FrictionCoefficient(InFrictionCoefficient)
			, Stiffness(InStiffness)
			, ColliderThickness(InColliderThickness)
		{
			check(!Thickness.HasWeightMap() || Thickness.Num() == NumParticles);
			check(!FrictionCoefficient.HasWeightMap() || FrictionCoefficient.Num() == NumParticles);
		}

		void Reset()
		{
			CollidingParticles.Reset();
			CollidingElements.Reset();
			Timers.Init(TMap<int32, FSolverReal>(), NumParticles);
		}

		void SetGeometry(const FTriangleMesh& InTriangleMesh, const TConstArrayView<FSolverVec3>& InPositions, const TConstArrayView<FSolverVec3>& InVelocities, const FTriangleMesh::TSpatialHashType<FSolverReal>& InSpatialHash)
		{
			TriangleMesh = &InTriangleMesh;
			Positions = InPositions;
			PAndInvM = TConstArrayView<FPAndInvM>();
			Velocities = InVelocities;
			SpatialHash = &InSpatialHash;
		}

		void SetGeometry(const FTriangleMesh& InTriangleMesh, const TConstArrayView<FSolverVec3>& InPositions, const TConstArrayView<FPAndInvM>& InPAndInvM, const TConstArrayView<FSolverVec3>& InVelocities, const FTriangleMesh::TSpatialHashType<FSolverReal>& InSpatialHash)
		{
			TriangleMesh = &InTriangleMesh;
			Positions = InPositions;
			PAndInvM = InPAndInvM;
			Velocities = InVelocities;
			SpatialHash = &InSpatialHash;
		}

		void SetStiffness(const FSolverReal InStiffness)
		{
			Stiffness = InStiffness;
		}

		void SetColliderThickness(const FSolverReal InColliderThickness)
		{
			ColliderThickness = InColliderThickness;
		}

		CHAOS_API void Init(const FSolverParticlesRange& Particles, const FSolverReal Dt);

		CHAOS_API void Apply(FSolverParticlesRange& InParticles, const FSolverReal Dt) const;

		const TArray<int32>& GetCollidingParticles() const { return CollidingParticles; }
		const TArray<TVector<int32, MaxKinematicConnectionsPerPoint>>& GetCollidingElements() const { return CollidingElements; }
		const TArray<TMap<int32, FSolverReal>>& GetTimers() const { return Timers; }
		const FTriangleMesh* GetTriangleMesh() const { return TriangleMesh; }
	protected:
		int32 NumParticles;
		const FPBDFlatWeightMap& Thickness;
		const FPBDFlatWeightMap& FrictionCoefficient;
		FSolverReal Stiffness;

		FSolverReal ColliderThickness;

		const FTriangleMesh* TriangleMesh = nullptr;
		TConstArrayView<FSolverVec3> Positions; 
		TConstArrayView<FPAndInvM> PAndInvM;
		TConstArrayView<FSolverVec3> Velocities;
		const FTriangleMesh::TSpatialHashType<FSolverReal>* SpatialHash = nullptr;

	private:

		// Parallel arrays (for ISPC SOA)
		TArray<int32> CollidingParticles;
		TArray<TVector<int32, MaxKinematicConnectionsPerPoint>> CollidingElements;

		TArray<TMap<int32, FSolverReal>> Timers; // Keep constraints for a cvar-defined time after it's moved out of proximity. Helps reduce jitter.
	};

	class FPBDSkinnedTriangleMeshCollisions : public FPBDKinematicTriangleMeshCollisions
	{
	public:
		FPBDSkinnedTriangleMeshCollisions(
			int32 InNumParticles,
			const FSkinnedTriangleMeshPtr& InSkinnedTriangleMesh,
			const TArray<FSolverVec3>& InVelocities,
			const FPBDFlatWeightMap& InThickness,
			const FPBDFlatWeightMap& InFrictionCoefficient,
			const FSolverReal InColliderThickness)
			: FPBDKinematicTriangleMeshCollisions(
				InNumParticles,
				InThickness,
				InFrictionCoefficient,
				1.f,
				InColliderThickness)
			, SkinnedTriangleMesh(InSkinnedTriangleMesh)
			, Velocities(InVelocities)
		{
		}

		void SetGeometryAndInit(const FSolverParticlesRange& Particles, const FSolverReal Dt)
		{
			if (SkinnedTriangleMesh)
			{
				SetGeometry(SkinnedTriangleMesh->GetTriangleMesh(), SkinnedTriangleMesh->GetLocalPositions(), TConstArrayView<FSolverVec3>(Velocities), SkinnedTriangleMesh->GetSpatialHierarchy());
				Init(Particles, Dt);
			}
		}

	private:
		const FSkinnedTriangleMeshPtr SkinnedTriangleMesh;
		const TArray<FSolverVec3>& Velocities;
	};

	class FPBDSkinnedTriangleMeshCollisionConstraints
	{
	public:
		static constexpr FSolverReal DefaultClothCollisionThickness = (FSolverReal)0.;
		static constexpr FSolverReal DefaultFrictionCoefficient = (FSolverReal)0.8;

		FPBDSkinnedTriangleMeshCollisionConstraints(int32 InNumParticles,
			const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
			const FCollectionPropertyConstFacade& PropertyCollection)
			: NumParticles(InNumParticles)
			, ClothCollisionThickness(FSolverVec2::Max(FSolverVec2(GetWeightedFloatClothCollisionThickness(PropertyCollection, DefaultClothCollisionThickness)), FSolverVec2(0.f)),
				WeightMaps.FindRef(GetClothCollisionThicknessString(PropertyCollection, ClothCollisionThicknessName.ToString())), 
				NumParticles)
			, FrictionCoefficient(FSolverVec2(GetWeightedFloatFrictionCoefficient(PropertyCollection, DefaultFrictionCoefficient)).ClampAxes(0.f, 1.f),
				WeightMaps.FindRef(GetFrictionCoefficientString(PropertyCollection, FrictionCoefficientName.ToString())), 
				NumParticles)
			, CollisionThickness(GetCollisionThickness(PropertyCollection,1.f))
			, bEnableSkinnedTriangleMeshCollisions(GetEnableSkinnedTriangleMeshCollisions(PropertyCollection, true))
			, bUseSelfCollisionSubstepsForSkinnedTriangleMeshes(GetUseSelfCollisionSubstepsForSkinnedTriangleMeshes(PropertyCollection, true))
			, EnableSkinnedTriangleMeshCollisionsIndex(PropertyCollection)
			, UseSelfCollisionSubstepsForSkinnedTriangleMeshesIndex(PropertyCollection)
			, ClothCollisionThicknessIndex(PropertyCollection)
			, FrictionCoefficientIndex(PropertyCollection)
			, CollisionThicknessIndex(PropertyCollection)
		{}

		void AddSkinnedTriangleMesh(const FParticleRangeIndex& Index, const FSkinnedTriangleMeshPtr& SkinnedTriangleMesh, const TArray<FSolverVec3>& Velocities)
		{
			Collisions.Emplace(Index, FPBDSkinnedTriangleMeshCollisions(NumParticles, SkinnedTriangleMesh, Velocities, ClothCollisionThickness, FrictionCoefficient, CollisionThickness));
		}

		void Init(const FSolverParticlesRange& Particles, const FSolverReal Dt)
		{
			if (!bEnableSkinnedTriangleMeshCollisions)
			{
				return;
			}
			for (TMap<FParticleRangeIndex, FPBDSkinnedTriangleMeshCollisions>::TIterator Iter = Collisions.CreateIterator(); Iter; ++Iter)
			{
				Iter.Value().SetGeometryAndInit(Particles, Dt);
			}
		}

		void Apply(FSolverParticlesRange& InParticles, const FSolverReal Dt) const
		{
			if (!bEnableSkinnedTriangleMeshCollisions)
			{
				return;
			}
			for (TMap<FParticleRangeIndex, FPBDSkinnedTriangleMeshCollisions>::TConstIterator Iter = Collisions.CreateConstIterator(); Iter; ++Iter)
			{
				Iter.Value().Apply(InParticles, Dt);
			}
		}

		void SetProperties(const FCollectionPropertyConstFacade& PropertyCollection,
			const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps)
		{
			if (IsEnableSkinnedTriangleMeshCollisionsMutable(PropertyCollection))
			{
				bEnableSkinnedTriangleMeshCollisions = GetEnableSkinnedTriangleMeshCollisions(PropertyCollection);
			}
			if (IsUseSelfCollisionSubstepsForSkinnedTriangleMeshesMutable(PropertyCollection))
			{
				bUseSelfCollisionSubstepsForSkinnedTriangleMeshes = GetUseSelfCollisionSubstepsForSkinnedTriangleMeshes(PropertyCollection);
			}
			if (IsClothCollisionThicknessMutable(PropertyCollection))
			{
				const FSolverVec2 WeightedValue = FSolverVec2::Max(FSolverVec2(GetWeightedFloatClothCollisionThickness(PropertyCollection)), FSolverVec2(0.f));
				if (IsClothCollisionThicknessStringDirty(PropertyCollection))
				{
					const FString& WeightMapName = GetClothCollisionThicknessString(PropertyCollection);
					ClothCollisionThickness = FPBDFlatWeightMap(WeightedValue, WeightMaps.FindRef(WeightMapName), NumParticles);
				}
				else
				{
					ClothCollisionThickness.SetWeightedValue(WeightedValue);
				}
			}

			if (IsFrictionCoefficientMutable(PropertyCollection))
			{
				const FSolverVec2 WeightedValue = FSolverVec2(GetWeightedFloatFrictionCoefficient(PropertyCollection)).ClampAxes(0.f, 1.f);
				if (IsFrictionCoefficientStringDirty(PropertyCollection))
				{
					const FString& WeightMapName = GetFrictionCoefficientString(PropertyCollection);
					FrictionCoefficient = FPBDFlatWeightMap(WeightedValue, WeightMaps.FindRef(WeightMapName), NumParticles);
				}
				else
				{
					FrictionCoefficient.SetWeightedValue(WeightedValue);
				}
			}
			if (IsCollisionThicknessMutable(PropertyCollection) && IsCollisionThicknessDirty(PropertyCollection))
			{
				CollisionThickness = FMath::Max(GetCollisionThickness(PropertyCollection), 0.f);
				for (TMap<FParticleRangeIndex, FPBDSkinnedTriangleMeshCollisions>::TIterator Iter = Collisions.CreateIterator(); Iter; ++Iter)
				{
					Iter.Value().SetColliderThickness(CollisionThickness);
				}
			}
		}

		void OnCollisionRangeRemoved(int32 CollisionRangeId)
		{
			for (TMap<FParticleRangeIndex, FPBDSkinnedTriangleMeshCollisions>::TIterator Iter = Collisions.CreateIterator(); Iter; ++Iter)
			{
				if (Iter.Key().RangeId == CollisionRangeId)
				{
					Iter.RemoveCurrent();
				}
			}
		}

		bool GetUseSelfCollisionSubstepsForSkinnedTriangleMeshes() const
		{
			return bUseSelfCollisionSubstepsForSkinnedTriangleMeshes;
		}

		bool IsEnabled() const
		{
			return bEnableSkinnedTriangleMeshCollisions;
		}

		FSolverReal GetCollisionThickness() const
		{
			return CollisionThickness;
		}

		FSolverReal GetMaxClothCollisionThickness() const
		{
			return FMath::Max(ClothCollisionThickness.GetLow(), ClothCollisionThickness.GetHigh());
		}

	private:
		int32 NumParticles;
		FPBDFlatWeightMap ClothCollisionThickness;
		FPBDFlatWeightMap FrictionCoefficient;
		FSolverReal CollisionThickness;
		bool bEnableSkinnedTriangleMeshCollisions = true;
		bool bUseSelfCollisionSubstepsForSkinnedTriangleMeshes = true;

		TMap<FParticleRangeIndex, FPBDSkinnedTriangleMeshCollisions> Collisions;

		UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(EnableSkinnedTriangleMeshCollisions, bool);
		UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(UseSelfCollisionSubstepsForSkinnedTriangleMeshes, bool);
		UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(ClothCollisionThickness, float);
		UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(FrictionCoefficient, float);
		UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(CollisionThickness, float);

	};

}  // End namespace Chaos::Softs

#if !defined(CHAOS_KINEMATIC_TRIANGLE_COLLISIONS_ISPC_ENABLED_DEFAULT)
#define CHAOS_KINEMATIC_TRIANGLE_COLLISIONS_ISPC_ENABLED_DEFAULT 1
#endif

// Support ISPC enable/disable in non-shipping builds
#if !INTEL_ISPC || UE_BUILD_SHIPPING
static constexpr bool bChaos_KinematicTriangleMesh_ISPC_Enabled = INTEL_ISPC && CHAOS_KINEMATIC_TRIANGLE_COLLISIONS_ISPC_ENABLED_DEFAULT;
#else
extern CHAOS_API bool bChaos_KinematicTriangleMesh_ISPC_Enabled;
#endif
