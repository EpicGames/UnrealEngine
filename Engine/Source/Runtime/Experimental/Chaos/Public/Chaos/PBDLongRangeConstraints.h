// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/PBDLongRangeConstraintsBase.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "ChaosStats.h"

DECLARE_CYCLE_STAT(TEXT("Chaos PBD Long Range Constraint"), STAT_PBD_LongRange, STATGROUP_Chaos);

namespace Chaos::Softs
{

class FPBDLongRangeConstraints : public FPBDLongRangeConstraintsBase
{
public:
	typedef FPBDLongRangeConstraintsBase Base;
	typedef typename Base::FTether FTether;

	static bool IsEnabled(const FCollectionPropertyConstFacade& PropertyCollection)
	{
		return IsTetherStiffnessEnabled(PropertyCollection, false);
	}

	FPBDLongRangeConstraints(
		const FSolverParticlesRange& Particles,
		const TArray<TConstArrayView<TTuple<int32, int32, FRealSingle>>>& InTethers,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
		const FCollectionPropertyConstFacade& PropertyCollection,
		FSolverReal MeshScale)
		: FPBDLongRangeConstraintsBase(
			Particles,
			InTethers,
			WeightMaps.FindRef(GetTetherStiffnessString(PropertyCollection, TetherStiffnessName.ToString())),
			WeightMaps.FindRef(GetTetherScaleString(PropertyCollection, TetherScaleName.ToString())),
			FSolverVec2(GetWeightedFloatTetherStiffness(PropertyCollection, 1.f)),
			FSolverVec2(GetWeightedFloatTetherScale(PropertyCollection, 1.f)),  // Scale clamping done in constructor
			FPBDStiffness::DefaultPBDMaxStiffness,
			MeshScale)
		, TetherStiffnessIndex(PropertyCollection)
		, TetherScaleIndex(PropertyCollection)
	{}

	FPBDLongRangeConstraints(
		const FSolverParticles& Particles,
		const int32 InParticleOffset,
		const int32 InParticleCount,
		const TArray<TConstArrayView<TTuple<int32, int32, FRealSingle>>>& InTethers,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
		const FCollectionPropertyConstFacade& PropertyCollection,
		FSolverReal MeshScale)
		: FPBDLongRangeConstraintsBase(
			Particles,
			InParticleOffset,
			InParticleCount,
			InTethers,
			WeightMaps.FindRef(GetTetherStiffnessString(PropertyCollection, TetherStiffnessName.ToString())),
			WeightMaps.FindRef(GetTetherScaleString(PropertyCollection, TetherScaleName.ToString())),
			FSolverVec2(GetWeightedFloatTetherStiffness(PropertyCollection, 1.f)),
			FSolverVec2(GetWeightedFloatTetherScale(PropertyCollection, 1.f)),  // Scale clamping done in constructor
			FPBDStiffness::DefaultPBDMaxStiffness,
			MeshScale)
		, TetherStiffnessIndex(PropertyCollection)
		, TetherScaleIndex(PropertyCollection)
	{}

	FPBDLongRangeConstraints(
		const FSolverParticles& Particles,
		const int32 InParticleOffset,
		const int32 InParticleCount,
		const TArray<TConstArrayView<TTuple<int32, int32, FRealSingle>>>& InTethers,
		const TConstArrayView<FRealSingle>& StiffnessMultipliers,
		const TConstArrayView<FRealSingle>& ScaleMultipliers,
		const FSolverVec2& InStiffness = FSolverVec2::UnitVector,
		const FSolverVec2& InScale = FSolverVec2::UnitVector,
		FSolverReal MeshScale = (FSolverReal)1.)
		: FPBDLongRangeConstraintsBase(
			Particles,
			InParticleOffset,
			InParticleCount,
			InTethers,
			StiffnessMultipliers,
			ScaleMultipliers,
			InStiffness,
			InScale,
			MeshScale)
		, TetherStiffnessIndex(ForceInit)
		, TetherScaleIndex(ForceInit)
	{}

	virtual ~FPBDLongRangeConstraints() override = default;

	using Base::SetProperties;

	CHAOS_API void SetProperties(
		const FCollectionPropertyConstFacade& PropertyCollection,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
		FSolverReal MeshScale);

	template<typename SolverParticlesOrRange>
	CHAOS_API void Apply(SolverParticlesOrRange& Particles, const FSolverReal Dt) const;

	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(TetherStiffness, float);
	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(TetherScale, float);

private:
	using Base::MinTetherScale;
	using Base::MaxTetherScale;
	using Base::Tethers;
	using Base::Stiffness;
	using Base::TetherScale;
	using Base::ParticleOffset;
	using Base::ParticleCount;

	UE_CHAOS_DECLARE_INDEXED_PROPERTYCOLLECTION_NAME(TetherStiffness, float);
	UE_CHAOS_DECLARE_INDEXED_PROPERTYCOLLECTION_NAME(TetherScale, float);
};

}  // End namespace Chaos::Softs

#if !defined(CHAOS_LONG_RANGE_ISPC_ENABLED_DEFAULT)
#define CHAOS_LONG_RANGE_ISPC_ENABLED_DEFAULT 1
#endif

#if !defined(USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING)
#define USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING 0
#endif

// Support run-time toggling on supported platforms in non-shipping configurations
#if !INTEL_ISPC || (UE_BUILD_SHIPPING && !USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING)
static constexpr bool bChaos_LongRange_ISPC_Enabled = INTEL_ISPC && CHAOS_LONG_RANGE_ISPC_ENABLED_DEFAULT;
#else
extern CHAOS_API bool bChaos_LongRange_ISPC_Enabled;
#endif
