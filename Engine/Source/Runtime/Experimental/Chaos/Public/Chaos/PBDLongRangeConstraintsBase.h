// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/PBDSoftsSolverParticles.h"
#include "Chaos/PBDSoftsEvolutionFwd.h"
#include "Chaos/PBDActiveView.h"
#include "Chaos/PBDStiffness.h"
#include "Chaos/SoftsSolverParticlesRange.h"
#include "Containers/Map.h"
#include "Containers/Set.h"

namespace Chaos::Softs
{

class FPBDLongRangeConstraintsBase
{
public:
	UE_NONCOPYABLE(FPBDLongRangeConstraintsBase);

	static constexpr FSolverReal MinTetherScale = (FSolverReal)0.01;
	static constexpr FSolverReal MaxTetherScale = (FSolverReal)10.;

	typedef TTuple<int32, int32, FRealSingle> FTether;

	CHAOS_API FPBDLongRangeConstraintsBase(
		const FSolverParticlesRange& Particles,
		const TArray<TConstArrayView<TTuple<int32, int32, FRealSingle>>>& InTethers,
		const TConstArrayView<FRealSingle>& StiffnessMultipliers,
		const TConstArrayView<FRealSingle>& ScaleMultipliers,
		const FSolverVec2& InStiffness = FSolverVec2::UnitVector,
		const FSolverVec2& InScale = FSolverVec2::UnitVector,
		FSolverReal MaxStiffness = FPBDStiffness::DefaultPBDMaxStiffness,
		FSolverReal MeshScale = (FSolverReal)1.);

	CHAOS_API FPBDLongRangeConstraintsBase(
		const FSolverParticles& Particles,
		const int32 InParticleOffset,
		const int32 InParticleCount,
		const TArray<TConstArrayView<TTuple<int32, int32, FRealSingle>>>& InTethers,
		const TConstArrayView<FRealSingle>& StiffnessMultipliers,
		const TConstArrayView<FRealSingle>& ScaleMultipliers,
		const FSolverVec2& InStiffness = FSolverVec2::UnitVector,
		const FSolverVec2& InScale = FSolverVec2::UnitVector,
		FSolverReal MaxStiffness = FPBDStiffness::DefaultPBDMaxStiffness,
		FSolverReal MeshScale = (FSolverReal)1.);

PRAGMA_DISABLE_DEPRECATION_WARNINGS  // For ScaleIndices and ScaleTable
	virtual ~FPBDLongRangeConstraintsBase() = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Return the stiffness input values used by the constraint
	FSolverVec2 GetStiffness() const { return Stiffness.GetWeightedValue(); }

	// Set the stiffness and scale values used by the constraint
	void SetProperties(const FSolverVec2& InStiffness, const FSolverVec2& InTetherScale, FSolverReal MeshScale = (FSolverReal)1.)
	{
		Stiffness.SetWeightedValue(InStiffness);
		TetherScale.SetWeightedValue(InTetherScale.ClampAxes(MinTetherScale, MaxTetherScale) * MeshScale);
	}

	// Set stiffness offset and range, as well as the simulation stiffness exponent
	void ApplyProperties(const FSolverReal Dt, const int32 NumIterations)
	{
		Stiffness.ApplyPBDValues(Dt, NumIterations);
		TetherScale.ApplyValues();
	}

	// Return the tethers, organized in concurent friendly batches
	const TArray<TConstArrayView<FTether>>& GetTethers() const { return Tethers; }

	// Return the start index of the specified tether
	int32 GetStartIndex(const FTether& Tether) const { return Tether.Get<0>(); }

	// Return the kinematic particle index of the specified tether
	int32 GetStartParticle(const FTether& Tether) const { return GetStartIndex(Tether) + ParticleOffset; }

	// Return the end index of the specified tether
	int32 GetEndIndex(const FTether& Tether) const { return Tether.Get<1>(); }

	// Return the dynamic particle index of the specified tether
	int32 GetEndParticle(const FTether& Tether) const { return GetEndIndex(Tether) + ParticleOffset; }

	// Return the reference length of the specified tether
	FSolverReal GetRefLength(const FTether& Tether) const { return (FSolverReal)Tether.Get<2>(); }

	// Return the Tether scale for the specified tether
	FSolverReal GetScale(const FTether& Tether) const { return TetherScale.HasWeightMap() ? TetherScale[GetEndIndex(Tether)] : (FSolverReal)TetherScale; }

	// Return the target length of the specified tether (= RefLength * Scale)
	FSolverReal GetTargetLength(const FTether& Tether) const { return GetRefLength(Tether) * GetScale(Tether); }

protected:
	// Return the minimum number of long range tethers in a batch to process in parallel
	static CHAOS_API int32 GetMinParallelBatchSize();

	// Return a vector representing the amount of segment required for the tether to shrink back to its maximum target length constraint, or zero if the constraint is already met
	template<typename SolverParticlesOrRange>
	inline FSolverVec3 GetDelta(const SolverParticlesOrRange& Particles, const FTether& Tether, const FSolverReal InScale) const
	{
		const int32 Start = GetStartParticle(Tether);
		const int32 End = GetEndParticle(Tether);
		const FSolverReal TargetLength = GetRefLength(Tether) * InScale;
		checkSlow(Particles.InvM(Start) == (FSolverReal)0.);
		checkSlow(Particles.InvM(End) > (FSolverReal)0.);
		FSolverVec3 Direction = Particles.P(Start) - Particles.P(End);
		const FSolverReal Length = Direction.SafeNormalize();
		const FSolverReal Offset = Length - TargetLength;
		return Offset < (FSolverReal)0. ? FSolverVec3((FSolverReal)0.) : Offset * Direction;
	};

	// Return a direction and length representing the amount of segment required for the tether to shrink back to its maximum target length constraint, or zero if the constraint is already met
	template<typename SolverParticlesOrRange>
	inline void GetDelta(const SolverParticlesOrRange& Particles, const FTether& Tether, const FSolverReal InScale, FSolverVec3& OutDirection, FSolverReal& OutOffset) const
	{
		const int32 Start = GetStartParticle(Tether);
		const int32 End = GetEndParticle(Tether);
		const FSolverReal TargetLength = GetRefLength(Tether) * InScale;
		checkSlow(Particles.InvM(Start) == (FSolverReal)0.);
		checkSlow(Particles.InvM(End) > (FSolverReal)0.);
		OutDirection = Particles.P(Start) - Particles.P(End);
		const FSolverReal Length = OutDirection.SafeNormalize();
		OutOffset = FMath::Max((FSolverReal)0., Length - TargetLength);
	};

protected:
	static constexpr int32 TableSize = 16;  // The size of the weightmaps lookup table
	const TArray<TConstArrayView<FTether>>& Tethers;  // Array view on the tether provided to this constraint
	const int32 ParticleOffset;  // Index of the first usable particle
	const int32 ParticleCount;
	FPBDStiffness Stiffness;  // Stiffness weightmap lookup table
	FPBDWeightMap TetherScale;  // Scale weightmap lookup table
};
}  // End namespace Chaos::Softs
