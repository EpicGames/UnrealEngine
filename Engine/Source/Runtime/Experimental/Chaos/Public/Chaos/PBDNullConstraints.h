// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Declares.h"
#include "Chaos/Evolution/IndexedConstraintContainer.h"
#include "Chaos/ParticleHandleFwd.h"


// @todo(chaos): These classes should be in the unit testing suite, but we must currently explicitly instantiate the ConstraintRules in the Chaos unit
// because the template code is in a source file. We need to support custom constraints anyway and the NullConstraint could use that when it exists.

namespace Chaos
{
	class FPBDNullConstraintHandle;
	class FPBDNullConstraints;

	/**
	 * @brief A dummy constraint used for unit testing
	*/
	class FPBDNullConstraint
	{
	public:
		FPBDNullConstraint(const TVec2<FGeometryParticleHandle*>& InConstrainedParticles)
			: ConstrainedParticles(InConstrainedParticles)
			, bEnabled(true)
			, bSleeping(false)
		{
		}

		FParticlePair ConstrainedParticles;
		bool bEnabled;
		bool bSleeping;
	};

	/**
	 * Constraint Container with minimal API required to test the Graph.
	 */
	class FPBDNullConstraints : public TPBDIndexedConstraintContainer<FPBDNullConstraints>
	{
	public:
		using FConstraintContainerHandle = FPBDNullConstraintHandle;

		FPBDNullConstraints();

		int32 NumConstraints() const { return Constraints.Num(); }

		FPBDNullConstraint& GetConstraint(const int32 ConstraintIndex)
		{
			return Constraints[ConstraintIndex];
		}

		const FPBDNullConstraint& GetConstraint(const int32 ConstraintIndex) const
		{
			return Constraints[ConstraintIndex];
		}

		FPBDNullConstraintHandle* AddConstraint(const TVec2<FGeometryParticleHandle*>& InConstraintedParticles);

		FParticlePair GetConstrainedParticles(int32 ConstraintIndex) const
		{
			return Constraints[ConstraintIndex].ConstrainedParticles;
		}

		TArray<FPBDNullConstraintHandle*>& GetConstraintHandles()
		{
			return Handles;
		}

		const TArray<FPBDNullConstraintHandle*>& GetConstraintHandles() const
		{
			return Handles;
		}

		//
		// FConstraintContainer Implementation
		//
		virtual int32 GetNumConstraints() const override final { return NumConstraints(); }
		virtual void ResetConstraints() override final {}
		virtual void AddConstraintsToGraph(Private::FPBDIslandManager& IslandManager) override final;
		virtual void PrepareTick() override final {}
		virtual void UnprepareTick() override final {}

		//
		// TSimpleConstraintContainerSolver API - used by RBAN solvers
		//
		void AddBodies(FSolverBodyContainer& SolverBodyContainer) {}
		void GatherInput(const FReal Dt) {}
		void ScatterOutput(const FReal Dt) {}
		void ApplyPositionConstraints(const FReal Dt, const int32 It, const int32 NumIts) {}
		void ApplyVelocityConstraints(const FReal Dt, const int32 It, const int32 NumIts) {}
		void ApplyProjectionConstraints(const FReal Dt, const int32 It, const int32 NumIts) {}

		//
		// TIndexedConstraintContainerSolver API - used by World solvers
		//
		void AddBodies(const TArrayView<int32>& ConstraintIndices, FSolverBodyContainer& SolverBodyContainer) {}
		void GatherInput(const TArrayView<int32>& ConstraintIndices, const FReal Dt) {}
		void ScatterOutput(const TArrayView<int32>& ConstraintIndices, const FReal Dt) {}
		void ApplyPositionConstraints(const TArrayView<int32>& ConstraintIndices, const FReal Dt, const int32 It, const int32 NumIts) {}
		void ApplyVelocityConstraints(const TArrayView<int32>& ConstraintIndices, const FReal Dt, const int32 It, const int32 NumIts) {}
		void ApplyProjectionConstraints(const TArrayView<int32>& ConstraintIndices, const FReal Dt, const int32 It, const int32 NumIts) {}

		TArray<FPBDNullConstraint> Constraints;
		TArray<FPBDNullConstraintHandle*> Handles;
		TConstraintHandleAllocator<FPBDNullConstraints> HandleAllocator;
	};

	class FPBDNullConstraintHandle final : public TIndexedContainerConstraintHandle<FPBDNullConstraints>
	{
	public:
		FPBDNullConstraintHandle(FPBDNullConstraints* InConstraintContainer, int32 ConstraintIndex)
			: TIndexedContainerConstraintHandle<FPBDNullConstraints>(InConstraintContainer, ConstraintIndex)
		{
		}

		virtual void SetEnabled(bool bInEnabled)  override
		{
			ConcreteContainer()->GetConstraint(GetConstraintIndex()).bEnabled = bInEnabled;
		}

		virtual bool IsEnabled() const override
		{
			return ConcreteContainer()->GetConstraint(GetConstraintIndex()).bEnabled;
		}

		virtual void SetIsSleeping(bool bInIsSleeping)  override
		{
			ConcreteContainer()->GetConstraint(GetConstraintIndex()).bSleeping = bInIsSleeping;
		}

		virtual bool IsSleeping() const override
		{
			return ConcreteContainer()->GetConstraint(GetConstraintIndex()).bSleeping;
		}

		virtual FParticlePair GetConstrainedParticles() const override
		{
			return ConcreteContainer()->GetConstrainedParticles(GetConstraintIndex());
		}

		static const FConstraintHandleTypeID& StaticType()
		{
			static FConstraintHandleTypeID STypeID(TEXT("FPBDNullConstraintHandle"), &FIndexedConstraintHandle::StaticType());
			return STypeID;
		}
	};


}