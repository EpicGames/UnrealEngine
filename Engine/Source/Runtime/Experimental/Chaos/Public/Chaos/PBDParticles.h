// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Real.h"
#include "Chaos/ArrayCollectionArray.h"
#include "Chaos/DynamicParticles.h"

namespace Chaos
{
template<class T, int d>
class TPBDParticles : public TDynamicParticles<T, d>
{
  public:
	TPBDParticles()
	    : TDynamicParticles<T, d>()
	{
		TArrayCollection::AddArray(&MP);
	}
	TPBDParticles(const TPBDParticles<T, d>& Other) = delete;
	TPBDParticles(TPBDParticles<T, d>&& Other)
	    : TDynamicParticles<T, d>(MoveTemp(Other)), MP(MoveTemp(Other.MP))
	{
		TArrayCollection::AddArray(&MP);
	}
	~TPBDParticles() = default;

	const TVector<T, d>& P(const int32 index) const { return MP[index]; }
	TVector<T, d>& P(const int32 index) { return MP[index]; }
	const TVector<T, d>& GetP(const int32 index)const  { return MP[index]; }
	void SetP(const int32 index, const TVector<T, d>& InP) { MP[index] = InP; }
	TArrayCollectionArray<TVector<T, d>>& GetP() { return MP; }


  private:
	TArrayCollectionArray<TVector<T, d>> MP;
};

using FPBDParticles = TPBDParticles<FReal, 3>;
}
