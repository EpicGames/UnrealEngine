// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Chaos/Core.h"
#include "Chaos/PBDSoftsEvolutionFwd.h"
#include "Chaos/PBDSoftsSolverParticles.h"
#include "Chaos/SoftsSolverParticlesRange.h"

namespace Chaos::Softs
{
	/** Simple planar constraints with friction. This assumes that the Indices are unique and thus can be solved in parallel.*/
	class FPBDPlanarConstraints
	{
	public:
		FPBDPlanarConstraints() = default;
		virtual ~FPBDPlanarConstraints() = default;
		FPBDPlanarConstraints(FSolverReal InFrictionCoefficient)
			: FrictionCoefficient(InFrictionCoefficient)
		{}

		void SetFrictionCoefficient(const FSolverReal InFrictionCoefficient) { FrictionCoefficient = InFrictionCoefficient; }
		void SetCollisionData(TArray<int32>&& InUniqueConstraintIndices, TArray<FSolverVec3>&& InTargetPositions, TArray<FSolverVec3>&& InTargetNormals, TArray<FSolverVec3>&& InTargetVelocities)
		{
			UniqueConstraintIndices = MoveTemp(InUniqueConstraintIndices);
			TargetPositions = MoveTemp(InTargetPositions);
			TargetNormals = MoveTemp(InTargetNormals);
			TargetVelocities = MoveTemp(InTargetVelocities);
		}

		void Reset()
		{
			UniqueConstraintIndices.Reset();
			TargetPositions.Reset();
			TargetNormals.Reset();
			TargetVelocities.Reset();
		}

		CHAOS_API void Apply(FSolverParticlesRange& Particles, const FSolverReal Dt) const;

		const TArray<int32>& GetUniqueConstraintIndices() const { return UniqueConstraintIndices; }
		const TArray<FSolverVec3>& GetTargetPositions() const { return TargetPositions; }
		const TArray<FSolverVec3>& GetTargetNormals() const { return TargetNormals; }
		const TArray<FSolverVec3>& GetTargetVelocities() const { return TargetVelocities; }

	protected:

		FSolverReal FrictionCoefficient = (FSolverReal)0.;

		TArray<int32> UniqueConstraintIndices;
		TArray<FSolverVec3> TargetPositions;
		TArray<FSolverVec3> TargetNormals;
		TArray<FSolverVec3> TargetVelocities;
	private:

		void ApplyHelperWithFriction(FSolverParticlesRange& Particles, const FSolverReal Dt, const int32 ConstraintIndex) const;
		void ApplyHelperNoFriction(FSolverParticlesRange& Particles, const int32 ConstraintIndex) const;

	};
}  // End namespace Chaos::Softs

#if !defined(CHAOS_PLANAR_ISPC_ENABLED_DEFAULT)
#define CHAOS_PLANAR_ISPC_ENABLED_DEFAULT 1
#endif

// Support run-time toggling on supported platforms in non-shipping configurations
#if !INTEL_ISPC || UE_BUILD_SHIPPING
static constexpr bool bChaos_Planar_ISPC_Enabled = INTEL_ISPC && CHAOS_PLANAR_ISPC_ENABLED_DEFAULT;
#else
extern CHAOS_API bool bChaos_Planar_ISPC_Enabled;
#endif
