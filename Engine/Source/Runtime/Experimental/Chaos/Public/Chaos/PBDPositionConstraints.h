// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Array.h"
#include "Chaos/Evolution/IndexedConstraintContainer.h"
#include "Chaos/ParticleHandle.h"

namespace Chaos
{
	class FPBDPositionConstraints;

	class FPBDPositionConstraintHandle final : public TIndexedContainerConstraintHandle<FPBDPositionConstraints>
	{
	public:
		using Base = TIndexedContainerConstraintHandle<FPBDPositionConstraints>;
		using FConstraintContainer = FPBDPositionConstraints;
		using FGeometryParticleHandle = FGeometryParticleHandle;

		FPBDPositionConstraintHandle() {}
		FPBDPositionConstraintHandle(FConstraintContainer* InConstraintContainer, int32 InConstraintIndex) 
			: TIndexedContainerConstraintHandle<FPBDPositionConstraints>(InConstraintContainer, InConstraintIndex) {}
		
		virtual FParticlePair GetConstrainedParticles() const override;

		static const FConstraintHandleTypeID& StaticType()
		{
			static FConstraintHandleTypeID STypeID(TEXT("FPositionConstraintHandle"), &FIndexedConstraintHandle::StaticType());
			return STypeID;
		}

	protected:
		using Base::ConstraintIndex;
		using Base::ConcreteContainer;
	};

	//! Constraint a single particle to a world-space position
	class FPBDPositionConstraints : public TPBDIndexedConstraintContainer<FPBDPositionConstraints>
	{
	public:
		using Base = TPBDIndexedConstraintContainer<FPBDPositionConstraints>;
		using FConstraintContainerHandle = FPBDPositionConstraintHandle;
		using FConstraintHandleAllocator = TConstraintHandleAllocator<FPBDPositionConstraints>;
		using FHandles = TArray<FConstraintContainerHandle*>;

		FPBDPositionConstraints(const FReal InStiffness = (FReal)1.)
			: TPBDIndexedConstraintContainer<FPBDPositionConstraints>(FConstraintContainerHandle::StaticType())
			, Stiffness(InStiffness)
		{}

		FPBDPositionConstraints(TArray<FVec3>&& Locations, TArray<FPBDRigidParticleHandle*>&& InConstrainedParticles, const FReal InStiffness = (FReal)1.)
			: TPBDIndexedConstraintContainer<FPBDPositionConstraints>(FConstraintContainerHandle::StaticType())
			, Targets(MoveTemp(Locations)), ConstrainedParticles(MoveTemp(InConstrainedParticles)), Stiffness(InStiffness)
		{
			if (ConstrainedParticles.Num() > 0)
			{
				Handles.Reserve(ConstrainedParticles.Num());
				for (int32 ConstraintIndex = 0; ConstraintIndex < ConstrainedParticles.Num(); ++ConstraintIndex)
				{
					Handles.Add(HandleAllocator.AllocHandle(this, ConstraintIndex));
				}
				ConstraintSolverBodies.SetNumZeroed(ConstrainedParticles.Num());
			}
		}

		virtual ~FPBDPositionConstraints() = default;


		//
		// Constraint Container API
		//

		/**
		 * Get the number of constraints.
		 */
		int32 NumConstraints() const
		{
			return ConstrainedParticles.Num();
		}

		/**
		 * Add a constraint.
		 */
		FConstraintContainerHandle* AddConstraint(FPBDRigidParticleHandle* Particle, const FVec3& Position)
		{
			int32 NewIndex = Targets.Num();
			Targets.Add(Position);
			ConstrainedParticles.Add(Particle);
			ConstraintSolverBodies.Add(nullptr);
			Handles.Add(HandleAllocator.AllocHandle(this, NewIndex));
			return Handles[NewIndex];
		}

		/**
		 * Remove a constraint.
		 */
		void RemoveConstraint(int ConstraintIndex)
		{
			FConstraintContainerHandle* ConstraintHandle = Handles[ConstraintIndex];
			if (ConstraintHandle != nullptr)
			{
				// Release the handle for the freed constraint
				HandleAllocator.FreeHandle(ConstraintHandle);
				Handles[ConstraintIndex] = nullptr;
			}

			// Swap the last constraint into the gap to keep the array packed
			Targets.RemoveAtSwap(ConstraintIndex);
			ConstrainedParticles.RemoveAtSwap(ConstraintIndex);
			ConstraintSolverBodies.RemoveAtSwap(ConstraintIndex);
			Handles.RemoveAtSwap(ConstraintIndex);

			// Update the handle for the constraint that was moved
			if (ConstraintIndex < Handles.Num())
			{
				SetConstraintIndex(Handles[ConstraintIndex], ConstraintIndex);
			}
		}


		/**
		 * Disabled the specified constraint.
		 */
		void DisableConstraints(const TSet<FGeometryParticleHandle*>& RemovedParticles)
		{
			// @todo(chaos)
		}


		//
		// Constraint API
		//
		FHandles& GetConstraintHandles()
		{
			return Handles;
		}
		const FHandles& GetConstConstraintHandles() const
		{
			return Handles;
		}

		const FConstraintContainerHandle* GetConstraintHandle(int32 ConstraintIndex) const
		{
			return Handles[ConstraintIndex];
		}

		FConstraintContainerHandle* GetConstraintHandle(int32 ConstraintIndex)
		{
			return Handles[ConstraintIndex];
		}

		/**
		 * Get the particles that are affected by the specified constraint.
		 */
		TVec2<FGeometryParticleHandle*> GetConstrainedParticles(int32 ConstraintIndex) const
		{
			return { ConstrainedParticles[ConstraintIndex], nullptr };
		}

		/**
		 * Get the world-space constraint positions for each body.
		 */
		const FVec3& GetConstraintPosition(int ConstraintIndex) const
		{
			return Targets[ConstraintIndex];
		}

		// @todo(ccaulfield): remove/rename
		void Replace(const int32 ConstraintIndex, const FVec3& Position)
		{
			Targets[ConstraintIndex] = Position;
		}

		void UpdatePositionBasedState(const FReal Dt) {}

		//
		// FConstraintContainer Implementation
		//
		virtual int32 GetNumConstraints() const override final { return NumConstraints(); }
		virtual void ResetConstraints() override final {}
		virtual void AddConstraintsToGraph(Private::FPBDIslandManager& IslandManager) override final;
		virtual void PrepareTick() override final {}
		virtual void UnprepareTick() override final {}

		//
		// TSimpleConstraintContainerSolver API - used by RBAN solvers
		//
		void AddBodies(FSolverBodyContainer& SolverBodyContainer);
		void GatherInput(const FReal Dt) {}
		void ScatterOutput(const FReal Dt);
		void ApplyPositionConstraints(const FReal Dt, const int32 It, const int32 NumIts);
		void ApplyVelocityConstraints(const FReal Dt, const int32 It, const int32 NumIts) {}
		void ApplyProjectionConstraints(const FReal Dt, const int32 It, const int32 NumIts) {}

		//
		// TIndexedConstraintContainerSolver API - used by World solvers
		//
		void AddBodies(const TArrayView<int32>& ConstraintIndices, FSolverBodyContainer& SolverBodyContainer);
		void GatherInput(const TArrayView<int32>& ConstraintIndices, const FReal Dt) {}
		void ScatterOutput(const TArrayView<int32>& ConstraintIndices, const FReal Dt);
		void ApplyPositionConstraints(const TArrayView<int32>& ConstraintIndices, const FReal Dt, const int32 It, const int32 NumIts);
		void ApplyVelocityConstraints(const TArrayView<int32>& ConstraintIndices, const FReal Dt, const int32 It, const int32 NumIts) {}
		void ApplyProjectionConstraints(const TArrayView<int32>& ConstraintIndices, const FReal Dt, const int32 It, const int32 NumIts) {}

	protected:
		using Base::GetConstraintIndex;
		using Base::SetConstraintIndex;

	private:
		void AddBodies(const int32 ConstraintIndex, FSolverBodyContainer& SolverBodyContainer);

		void ApplySingle(const FReal Dt, int32 ConstraintIndex) const
		{
			FSolverBody* Body = ConstraintSolverBodies[ConstraintIndex];
			if (Body != nullptr)
			{
				const FVec3& P1 = Body->CorrectedP();
				const FVec3& P2 = Targets[ConstraintIndex];
				const FVec3 Difference = P1 - P2;
				Body->ApplyPositionDelta(-Stiffness * Difference);
			}
		}

		TArray<FVec3> Targets;
		TArray<FPBDRigidParticleHandle*> ConstrainedParticles;
		FReal Stiffness;

		TArray<FSolverBody*> ConstraintSolverBodies;

		FHandles Handles;
		FConstraintHandleAllocator HandleAllocator;
	};

	template <typename T, int d>
	using TPBDPositionConstraintHandle UE_DEPRECATED(4.27, "Deprecated. this class is to be deleted, use FPBDPositionConstraintHandle instead") = FPBDPositionConstraintHandle;

	template <typename T, int d>
	using TPBDPositionConstraints UE_DEPRECATED(4.27, "Deprecated. this class is to be deleted, use FPBDPositionConstraints instead") = FPBDPositionConstraints;

}
