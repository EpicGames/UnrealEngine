// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/ArrayCollectionArray.h"
#include "Chaos/PBDRigidParticles.h"
#include "Chaos/ImplicitObjectUnion.h"
#include "Templates/PimplPtr.h"

namespace Chaos
{

namespace Private
{
	class FConvexOptimizer;
}

/** 
 * Used within the clustering system to describe the clustering hierarchy. The ClusterId
 * stores the children IDs, and a Parent ID. When Id == \c nullptr the cluster is not
 * controlled by another body. 
 *
 * TODO: Chaos - Add dimension template param?  Add floating point param?
 */
struct ClusterId
{
	ClusterId() 
		: Id(nullptr)
		, NumChildren(0) 
	{}
	ClusterId(FPBDRigidParticleHandle* NewId, int NumChildrenIn)
		: Id(NewId)
		, NumChildren(NumChildrenIn) 
	{}
	FPBDRigidParticleHandle* Id;
	int32 NumChildren;
};


/**
 * An entry in a clustered particle's \c ConnectivityEdges array, indicating a
 * connection between that body and \c Sibling, with a strength breakable by 
 * a \c Strain threshold.
 *
 * TODO: Chaos - Add dimension template param?
 */
template <typename T>
struct TConnectivityEdge
{
	TConnectivityEdge() 
		: Sibling(nullptr)
		, Strain(0.0)
	{}

	TConnectivityEdge(TPBDRigidParticleHandle<T, 3>* InSibling, const FRealSingle InStrain)
		: Sibling(InSibling)
		, Strain(InStrain) 
	{}

	TConnectivityEdge(const TConnectivityEdge& Other)
		: Sibling(Other.Sibling)
		, Strain(Other.Strain) 
	{}

	/** Compares by \p OtherSibling only, for \c TArray::FindByKey(). */
	bool operator==(const TPBDRigidParticleHandle<T, 3>* OtherSibling) const
	{ return Sibling == OtherSibling; }

	// getter/functions functions to make calling code less confusing 
	void SetArea(Chaos::FRealSingle Area) { Strain = Area; }
	Chaos::FRealSingle GetArea() const { return Strain; }

	TPBDRigidParticleHandle<T, 3>* Sibling;
	// this can be both strain or area based on the damage model, but cannot rename it for backward compatibility reasons
	Chaos::FRealSingle Strain; 
};
typedef TConnectivityEdge<FReal> FConnectivityEdge;
typedef TArray<FConnectivityEdge> FConnectivityEdgeArray;

template<typename T>
bool IsInterclusterEdge(const TPBDRigidParticleHandle<T, 3>& Particle, const TConnectivityEdge<T>& Edge)
{
	if (!Edge.Sibling)
	{
		return false;
	}

	const TPBDRigidClusteredParticleHandle<T, 3>* ClusterParticle = Particle.CastToClustered();
	const TPBDRigidClusteredParticleHandle<T, 3>* SiblingParticle = Edge.Sibling->CastToClustered();

	if (!ClusterParticle || !SiblingParticle)
	{
		return false;
	}

	return ClusterParticle->Parent() != SiblingParticle->Parent();
}

class FRigidClusteredFlags
{
public:
	using FStorage = uint8;

	FRigidClusteredFlags()
		: Bits(0)
	{
	}

	// set to true if the particle is an internal cluster formed of the subset of children from an original cluster 
	bool GetInternalCluster() const { return Flags.bInternalCluster; }
	void SetInternalCluster(bool bSet) { Flags.bInternalCluster = bSet; }

	// set to true make the particle act as a kinematic anchor, this allows the particle to be broken off while still be anchor contributor through the connection graph   
	bool GetAnchored() const { return Flags.bAnchored; }
	void SetAnchored(bool bSet) { Flags.bAnchored = bSet; }

	// set to true to make the particle unbreakable by destruction operations from breaking server authoritative particles
	bool GetUnbreakable() const { return Flags.bUnbreakable; }
	void SetUnbreakable(bool bSet) { Flags.bUnbreakable = bSet; }

	// Set to true if we want to soft lock the particle's ChildToParent transform. This prevents its from being updated in Chaos::UpdateGeometry.
	bool GetChildToParentLocked() const { return Flags.bChildToParentLocked; }
	void SetChildToParentLocked(bool bSet) { Flags.bChildToParentLocked = bSet; }

private:
	struct FFlags
	{
		FStorage bInternalCluster : 1;
		FStorage bAnchored : 1;
		FStorage bUnbreakable : 1;
		FStorage bChildToParentLocked : 1;

		// Add new properties above this line
		// Change FStorage typedef if we exceed the max bits
	};
	union
	{
		FFlags Flags;
		FStorage Bits;
	};
	static_assert(sizeof(FFlags) <= sizeof(FStorage));
};


template<class T, int d>
class TPBDRigidClusteredParticles : public TPBDRigidParticles<T, d>
{
  public:
	TPBDRigidClusteredParticles()
	: TPBDRigidParticles<T, d>()
	{
		InitHelper();
	}
	TPBDRigidClusteredParticles(const TPBDRigidParticles<T, d>& Other) = delete;
	TPBDRigidClusteredParticles(TPBDRigidParticles<T, d>&& Other)
	: TPBDRigidParticles<T, d>(MoveTemp(Other))
	, MClusterIds(MoveTemp(Other.MClusterIds))
	, MChildToParent(MoveTemp(Other.MChildToParent))
	, MClusterGroupIndex(MoveTemp(Other.MClusterGroupIndex))
	, MChildrenSpatial(MoveTemp(Other.MChildrenSpatial))
	, MPhysicsProxies(MoveTemp(Other.MPhysicsProxies))
	, MCollisionImpulses(MoveTemp(Other.MCollisionImpulses))
	, MStrains(MoveTemp(Other.MStrains))
	, MConnectivityEdges(MoveTemp(Other.MConnectivityEdges))
	, MExternalStrains(MoveTemp(Other.MExternalStrains))
	, MRigidClusteredFlags(MoveTemp(Other.MRigidClusteredFlags))
	, MConvexOptimizers(MoveTemp(Other.MConvexOptimizers))
	{
		InitHelper();
	}
	~TPBDRigidClusteredParticles() = default;

	const auto& ClusterIds(int32 Idx) const { return MClusterIds[Idx]; }
	auto& ClusterIds(int32 Idx) { return MClusterIds[Idx]; }

	const auto& ChildToParent(int32 Idx) const { return MChildToParent[Idx]; }
	auto& ChildToParent(int32 Idx) { return MChildToParent[Idx]; }

	const auto& ClusterGroupIndex(int32 Idx) const { return MClusterGroupIndex[Idx]; }
	auto& ClusterGroupIndex(int32 Idx) { return MClusterGroupIndex[Idx]; }

	const auto& GetChildrenSpatial(int32 Idx) const { return MChildrenSpatial[Idx]; }
	auto& GetChildrenSpatial(int32 Idx) { return MChildrenSpatial[Idx]; }
	
	UE_DEPRECATED(5.4, "Use GetChildrenSpatial instead")
	const TUniquePtr<FImplicitObjectUnionClustered>& ChildrenSpatial(int32 Idx) const
	{
		check(false);
		static TUniquePtr<FImplicitObjectUnionClustered> DummyPtr(nullptr);
		return DummyPtr;
	}
	
	UE_DEPRECATED(5.4, "Use GetChildrenSpatial instead")
    TUniquePtr<FImplicitObjectUnionClustered>& ChildrenSpatial(int32 Idx)
	{
		check(false);
		static TUniquePtr<FImplicitObjectUnionClustered> DummyPtr(nullptr);
		return DummyPtr; 
	}

	const auto& PhysicsProxies(int32 Idx) const { return MPhysicsProxies[Idx]; }
	auto& PhysicsProxies(int32 Idx) { return MPhysicsProxies[Idx]; }

	const auto& CollisionImpulses(int32 Idx) const { return MCollisionImpulses[Idx]; }
	auto& CollisionImpulses(int32 Idx) { return MCollisionImpulses[Idx]; }
	auto& CollisionImpulsesArray() { return MCollisionImpulses; }

	const auto& ExternalStrains(int32 Idx) const { return MExternalStrains[Idx]; }
	auto& ExternalStrains(int32 Idx) { return MExternalStrains[Idx]; }
	auto& ExternalStrainsArray() { return MExternalStrains; }
	
	const auto& Strains(int32 Idx) const { return MStrains[Idx]; }
	auto& Strains(int32 Idx) { return MStrains[Idx]; }

	const auto& ConnectivityEdges(int32 Idx) const { return MConnectivityEdges[Idx]; }
	auto& ConnectivityEdges(int32 Idx) { return MConnectivityEdges[Idx]; }

	const FRigidClusteredFlags& RigidClusteredFlags(int32 Idx) const { return MRigidClusteredFlags[Idx]; }
	FRigidClusteredFlags& RigidClusteredFlags(int32 Idx) { return MRigidClusteredFlags[Idx]; }
	
	const auto& ConnectivityEdgesArray() const { return MConnectivityEdges; }

	const auto& ClusterIdsArray() const { return MClusterIds; }
	auto& ClusterIdsArray() { return MClusterIds; }

	const auto& ChildToParentArray() const { return MChildToParent; }
	auto& ChildToParentArray() { return MChildToParent; }

	const auto& StrainsArray() const { return MStrains; }
	auto& StrainsArray() { return MStrains; }

	const auto& ClusterGroupIndexArray() const { return MClusterGroupIndex; }
	auto& ClusterGroupIndexArray() { return MClusterGroupIndex; }

	const auto& RigidClusteredFlags() const { return MRigidClusteredFlags; }
	auto& RigidClusteredFlags() { return MRigidClusteredFlags; }

	const auto& ConvexOptimizers(int32 Idx) const { return MConvexOptimizers[Idx]; }
	auto& ConvexOptimizers(int32 Idx) { return MConvexOptimizers[Idx]; }
	
	typedef TPBDRigidClusteredParticleHandle<T, d> THandleType;
	const THandleType* Handle(int32 Index) const { return static_cast<const THandleType*>(TGeometryParticles<T,d>::Handle(Index)); }

	//cannot be reference because double pointer would allow for badness, but still useful to have non const access to handle
	THandleType* Handle(int32 Index) { return static_cast<THandleType*>(TGeometryParticles<T, d>::Handle(Index)); }

	
  private:

	  void InitHelper()
	  {
		  this->MParticleType = EParticleType::Clustered;
		  TArrayCollection::AddArray(&MClusterIds);
		  TArrayCollection::AddArray(&MChildToParent);
		  TArrayCollection::AddArray(&MClusterGroupIndex);
		  TArrayCollection::AddArray(&MChildrenSpatial);
		  TArrayCollection::AddArray(&MPhysicsProxies);
		  TArrayCollection::AddArray(&MCollisionImpulses);
		  TArrayCollection::AddArray(&MStrains);
		  TArrayCollection::AddArray(&MConnectivityEdges);
	  	  TArrayCollection::AddArray(&MExternalStrains);
	  	  TArrayCollection::AddArray(&MRigidClusteredFlags);
	  	  TArrayCollection::AddArray(&MConvexOptimizers);
	  }

	  TArrayCollectionArray<ClusterId> MClusterIds;
	  TArrayCollectionArray<TRigidTransform<T, d>> MChildToParent;
	  TArrayCollectionArray<int32> MClusterGroupIndex;
	  TArrayCollectionArray<FImplicitObjectUnionClusteredPtr> MChildrenSpatial;

	  // Multiple proxy pointers required for internal clusters
	  TArrayCollectionArray<TSet<IPhysicsProxyBase*>> MPhysicsProxies;

	  // Collision Impulses
	  TArrayCollectionArray<Chaos::FRealSingle> MCollisionImpulses;

	  // external strains ( use by fields )
	  // @todo(chaos) we should eventually merge MCollisionImpulses into MExternalStrains when Clustering code has been updated to not clear the impulses just before processing them 
	  TArrayCollectionArray<Chaos::FRealSingle> MExternalStrains;

	  // User set parameters
	  TArrayCollectionArray<Chaos::FRealSingle> MStrains;

	  TArrayCollectionArray<TArray<TConnectivityEdge<T>>> MConnectivityEdges;
  
	  TArrayCollectionArray<FRigidClusteredFlags> MRigidClusteredFlags;

	  // Per clustered particle convex optimizer to reduce the collision cost
	  TArrayCollectionArray<TPimplPtr<Private::FConvexOptimizer>> MConvexOptimizers;
};

using FPBDRigidClusteredParticles = TPBDRigidClusteredParticles<FReal, 3>;

} // namespace Chaos
