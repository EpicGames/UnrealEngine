// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/ClusterUnionManager.h"
#include "ChaosSolverConfiguration.h"
#include "Chaos/PBDRigidClusteredParticles.h"
#include "Chaos/PBDCollisionConstraints.h"
#include "Chaos/Transform.h"
#include "Chaos/CollisionResolutionTypes.h"
#include "Chaos/ExternalCollisionData.h"
#include "Chaos/TriangleMesh.h"
#include "Chaos/ClusterCreationParameters.h"
#include "Chaos/GeometryParticlesfwd.h"
#include "Framework/BufferedData.h"
#include "Chaos/PBDRigidClusteringTypes.h"

namespace Chaos
{
	class FPBDCollisionConstraints;

struct FClusterDestoryParameters {
	bool bReturnInternalOnly : true;
};

/****
*
*   FRigidClustering
* 
*   The Chaos Destruction System allows artists to define exactly how geometry 
*   will break and separate during the simulations. Artists construct the 
*	simulation assets using pre-fractured geometry and utilize dynamically 
*   generated rigid constraints to model the structural connections during the
*   simulation. The resulting objects within the simulation can separate from 
*   connected structures based on interactions with environmental elements, 
*   like fields and collisions.
*
*   The destruction system relies on an internal clustering model 
*   (aka Clustering) which controls how the rigidly attached geometry is 
*   simulated. Clustering allows artists to initialize sets of geometry as 
*   a single rigid body, then dynamically break the objects during the 
*   simulation. At its core, the clustering system will simply join the mass
*   and inertia of each connected element into one larger single rigid body.
* 
*   At the beginning of the simulation a connection graph is initialized 
*   based on the rigid body’s nearest neighbors. Each connection between the
*   bodies represents a rigid constraint within the cluster and is given 
*   initial strain values. During the simulation, the strains within the 
*   connection graph are evaluated. The connections can be broken when collision
*   constraints, or field evaluations, impart an impulse on the rigid body that
*   exceeds the connections limit. Fields can also be used to decrease the 
*   internal strain values of the connections, resulting in a weakening of the
*   internal structure
*
*/
class FRigidClustering
{
public:

	typedef FPBDRigidsEvolutionGBF								FRigidEvolution;
	typedef FPBDRigidParticleHandle*							FRigidHandle;
	typedef TArray<FRigidHandle>								FRigidHandleArray;
	typedef FPBDRigidClusteredParticleHandle*					FClusterHandle;
	typedef TMap<FClusterHandle, FRigidHandleArray>				FClusterMap;
	typedef TFunction<void(FRigidClustering&, FRigidHandle)>	FVisitorFunction;

	CHAOS_API FRigidClustering(FRigidEvolution& InEvolution, FPBDRigidClusteredParticles& InParticles, const TArray<ISimCallbackObject*>* InStrainModifiers);
	CHAOS_API ~FRigidClustering();

	//
	// Initialization
	//

	/**
	 *  Initialize a cluster with the specified children.
	 *
	 *	\p ClusterGroupIndex - Index to join cluster into.
	 *	\p Children - List of children that should belong to the cluster
	 *	\p Parameters - ClusterParticleHandle must be valid, this is the parent cluster body.
	 *	ProxyGeometry : Collision default for the cluster, automatically generated otherwise.
	 *		ForceMassOrientation : Inertial alignment into mass space.
	 *	
	 */
	CHAOS_API Chaos::FPBDRigidClusteredParticleHandle* CreateClusterParticle(
		const int32 ClusterGroupIndex, 
		TArray<Chaos::FPBDRigidParticleHandle*>&& Children, 
		const FClusterCreationParameters& Parameters = FClusterCreationParameters(),
		const Chaos::FImplicitObjectPtr& ProxyGeometry = nullptr,
		const FRigidTransform3* ForceMassOrientation = nullptr,
		const FUniqueIdx* ExistingIndex = nullptr);

	UE_DEPRECATED(5.4, "Use CreateClusterParticle with FImplicitObjectPtr instead")
	Chaos::FPBDRigidClusteredParticleHandle* CreateClusterParticle(
    		const int32 ClusterGroupIndex, 
    		TArray<Chaos::FPBDRigidParticleHandle*>&& Children, 
    		const FClusterCreationParameters& Parameters = FClusterCreationParameters(),
    		TSharedPtr<Chaos::FImplicitObject, ESPMode::ThreadSafe> ProxyGeometry = nullptr,
    		const FRigidTransform3* ForceMassOrientation = nullptr,
    		const FUniqueIdx* ExistingIndex = nullptr)
	{
		check(false);
		return nullptr;
	}

	/**
	 *  CreateClusterParticleFromClusterChildren
	 *    Children : Rigid body ID to include in the cluster.
	 */
	CHAOS_API Chaos::FPBDRigidClusteredParticleHandle* CreateClusterParticleFromClusterChildren(
		TArray<FPBDRigidParticleHandle*>&& Children, 
		FPBDRigidClusteredParticleHandle* Parent,
		const FRigidTransform3& ClusterWorldTM, 
		const FClusterCreationParameters& Parameters/* = FClusterCreationParameters()*/);

	/**
	 * Manually add a set of particles to a cluster after the cluster has already been created.
	 *	ChildToParentMap: A map that may contain a pointer to one of the child particles as a key, and a pointer to its old parent particle (prior to be being released for example).
	 *					  If a child particle exists in this map, its parent (or whatever is specified as the value) will be used to determine the correct proxy to add to the parent cluster.
	 */
	CHAOS_API void AddParticlesToCluster(
		FPBDRigidClusteredParticleHandle* Cluster,
		const TArray<FPBDRigidParticleHandle*>& InChildren,
		const TMap<FPBDRigidParticleHandle*, FPBDRigidParticleHandle*>& ChildToParentMap);

	/**
	 * Manually remove a set of particles from the cluster.
	 */
	CHAOS_API void RemoveParticlesFromCluster(
		FPBDRigidClusteredParticleHandle* Cluster,
		const TArray<FPBDRigidParticleHandle*>& InChildren);
	/**
	 *  UnionClusterGroups
	 *    Clusters that share a group index should be unioned into a single cluster prior to simulation.
	 *    The GroupIndex should be set on creation, and never touched by the client again.
	 */
	CHAOS_API void UnionClusterGroups();

	//
	// Releasing
	//

	/*
	*  DeactivateClusterParticle
	*    Release all the particles within the cluster particle
	*/
	CHAOS_API TSet<FPBDRigidParticleHandle*> DeactivateClusterParticle(FPBDRigidClusteredParticleHandle* ClusteredParticle);

	/*
	*  ReleaseClusterParticles (BasedOnStrain)
	*    Release clusters based on the passed in \p ExternalStrainArray, or the 
	*    particle handle's current \c CollisionImpulses() value. Any cluster bodies 
	*    that have a strain value less than this valid will be released from the 
	*    cluster.
	*/
	CHAOS_API TSet<FPBDRigidParticleHandle*> ReleaseClusterParticles(
		FPBDRigidClusteredParticleHandle* ClusteredParticle, 
		bool bForceRelease = false);

	CHAOS_API TSet<FPBDRigidParticleHandle*> ReleaseClusterParticlesNoInternalCluster(
		FPBDRigidClusteredParticleHandle* ClusteredParticle,
		bool bForceRelease = false);

	/*
	*  ReleaseClusterParticles
	*    Release all rigid body IDs passed,
	*/
	CHAOS_API TSet<FPBDRigidParticleHandle*> ReleaseClusterParticles(
		TArray<FPBDRigidParticleHandle*> ChildrenParticles, bool bTriggerBreakEvents = false);

	/** 
	* Force Release a particle at any level by making sure their parent particles are also release if necessary 
	* @Warning this will force all particles including the ones in the parent chain to be made breakable 
	*/
	CHAOS_API void ForceReleaseChildParticleAndParents(FPBDRigidClusteredParticleHandle* ChildClusteredParticle, bool bTriggerBreakEvents);

	/*
	*  DestroyClusterParticle
	*    Disable the cluster particle and remove from all internal clustering
	*    structures. This will not activate children.
	* 
	*    Returns the active parent cluster that might need to be rebuilt because
	*    its geometry might be pointing to deleted particle handels. 
	*/
	CHAOS_API FPBDRigidClusteredParticleHandle* DestroyClusterParticle(
		FPBDRigidClusteredParticleHandle* ClusteredParticle,
		const FClusterDestoryParameters& Parameters = FClusterDestoryParameters());

	/*
	*  BreakCluster
	*    Breaks a cluster (internal or not) by applying max external strain to all its children 
	* 
	*    @param ClusteredParticle handle of the cluster to break
	*    @return true if the cluster was successfully found and act upon 
	*/
	CHAOS_API bool BreakCluster(FPBDRigidClusteredParticleHandle* ClusteredParticle);

	/*
	*  BreakClustersByProxy
	*    Breaks clusters (internal or not) own by a specific proxy by applying max external strain to all its children
	*
	*    @param Proxy proxy owning the clusters to break
	*    @return true if any cluster was successfully found and act upon
	*/
	CHAOS_API bool BreakClustersByProxy(const IPhysicsProxyBase* Proxy);
	
	//
	// Operational 
	//

	/*
	*  AdvanceClustering
	*   Advance the cluster forward in time;
	*   ... Union unprocessed geometry.
	*   ... Release bodies based collision impulses.
	*   ... Updating properties as necessary.
	*/
	CHAOS_API void AdvanceClustering(const FReal dt, FPBDCollisionConstraints& CollisionRule);

	/**
	*  BreakingModel
	*    Implements the promotion breaking model, where strain impulses are
	*    summed onto the cluster body, and released if greater than the
	*    encoded strain. The remainder strains are propagated back down to
	*    the children clusters.
	*/
	CHAOS_API void BreakingModel();
	CHAOS_API void BreakingModel(TArray<FPBDRigidClusteredParticleHandle*>& InParticles);
	CHAOS_API bool BreakingModel(TArrayView<FPBDRigidClusteredParticleHandle*> InParticles);
	
	//
	// Access
	//

	/**
	*
	*  Visitor
	*   Walk all the decendents of the current cluster and execute FVisitorFunction.
	*   FVisitorFunction = [](FRigidClustering& Clustering, FRigidHandle RigidHandle){}
	*/
	CHAOS_API void Visitor(FClusterHandle Cluster, FVisitorFunction Function);

	/*
	*  GetActiveClusterIndex
	*    Get the current childs active cluster. Returns INDEX_NONE if
	*    not active or driven.
	*/
	CHAOS_API FPBDRigidParticleHandle* GetActiveClusterIndex(FPBDRigidParticleHandle* Child);

	/*
	*  GetClusterIdsArray
	*    The cluster ids provide a mapping from the rigid body index
	*    to its parent cluster id. The parent id might not be the
	*    active id, see the GetActiveClusterIndex to find the active cluster.
	*    INDEX_NONE represents a non-clustered body.
	*/
	TArrayCollectionArray<ClusterId>& GetClusterIdsArray() { return MParticles.ClusterIdsArray(); }
	const TArrayCollectionArray<ClusterId>& GetClusterIdsArray() const { return MParticles.ClusterIdsArray(); }

	/*
	*  GetRigidClusteredFlagsArray
	*    The RigidClusteredFlags array contains various flag related to clustered particles
	*/
	const TArrayCollectionArray<FRigidClusteredFlags>& GetRigidClusteredFlagsArray() const { return MParticles.RigidClusteredFlags(); }

	/*
	*  GetChildToParentMap
	*    This map stores the relative transform from a child to its cluster parent.
	*/
	const TArrayCollectionArray<FRigidTransform3>& GetChildToParentMap() const { return MParticles.ChildToParentArray(); }

	/*
	*  GetStrainArray
	*    The strain array is used to store the maximum strain allowed for a individual
	*    body in the simulation. This attribute is initialized during the creation of
	*    the cluster body, can be updated during the evaluation of the simulation.
	*/
	TArrayCollectionArray<FRealSingle>& GetStrainArray() { return MParticles.StrainsArray(); }
	const TArrayCollectionArray<FRealSingle>& GetStrainArray() const { return MParticles.StrainsArray(); }
		
	/**
	*  GetParentToChildren
	*    The parent to children map stores the currently active cluster ids (Particle Indices) as
	*    the keys of the map. The value of the map is a pointer to an array  constrained
	*    rigid bodies.
	*/
	FClusterMap& GetChildrenMap() { return MChildren; }
	const FClusterMap& GetChildrenMap() const { return MChildren; }

	/*
	*  GetClusterGroupIndexArray
	*    The group index is used to automatically bind disjoint clusters. This attribute it set
	*    during the creation of cluster to a positive integer value. During UnionClusterGroups (which
	*    is called during AdvanceClustering) the positive bodies are joined with a negative pre-existing
	*    body, then set negative. Zero entries are ignored within the union.
	*/
	TArrayCollectionArray<int32>& GetClusterGroupIndexArray() { return MParticles.ClusterGroupIndexArray(); }

	/*
	* Reset all events ( this include breaking, crumbling event and tracking data 
	*/
	CHAOS_API void ResetAllEvents();

	/*
	*  Cluster Break Data
	*     The cluster breaks can be used to seed particle emissions. 
	*/
	const TArray<FBreakingData>& GetAllClusterBreakings() const { return MAllClusterBreakings; }
	void SetGenerateClusterBreaking(bool DoGenerate) { DoGenerateBreakingData = DoGenerate; }
	bool GetDoGenerateBreakingData() const { return DoGenerateBreakingData; }
	void ResetAllClusterBreakings() { MAllClusterBreakings.Reset(); }

	/*
	*  Cluster crumbling Data
	*     triggered when all the children of a cluster are released all at once
	*     event is generated only if the owning proxy allows it
	*/
	const TArray<FCrumblingData>& GetAllClusterCrumblings() const { return MAllClusterCrumblings; }
	void ResetAllClusterCrumblings() { MAllClusterCrumblings.Reset(); }

	/*
	* GetConnectivityEdges
	*    Provides a list of each rigid body's current siblings and associated strain within the cluster.
	*/
	const TArrayCollectionArray<TArray<TConnectivityEdge<FReal>>>& GetConnectivityEdges() const { return MParticles.ConnectivityEdgesArray(); }

	/*
	*  FindClosestChild
	*    Find the closest child of an active cluster from a world position
	*    current implementation will pick the closest based on the distance from the center
	*    future implementation may expose option for more precise queries
	*    @param ClusteredParticle active cluster handle to query the children from
	*    @param WorldLocation world space location to find the closest child from   
	*/
	CHAOS_API FPBDRigidParticleHandle* FindClosestChild(const FPBDRigidClusteredParticleHandle* ClusteredParticle, const FVec3& WorldLocation) const;

	/*
	*  FindClosest
	*    Find the closest particle from an array of  particle
	*    current implementation will pick the closest based on the distance from the center
	*    future implementation may expose option for more precise queries
	*    @param Particles array of clustered particles to select from  
	*    @param WorldLocation world space location to find the closest particle from   
	*/
	static CHAOS_API FPBDRigidParticleHandle* FindClosestParticle(const TArray<FPBDRigidParticleHandle*>& Particles, const FVec3& WorldLocation);

	/*
	*  FindChildrenWithinRadius
	*    Find the children of an active cluster from a world position and a radius
	*    current implementation will pick the closest based on the distance from the center
	*    future implementation may expose option for more precise queries
	*    if bAlwaysReturnClosest is checked this will always return  the closest from the location even if the radius does not encompass any center of mass  
	*    @param ClusteredParticle active cluster handle to query the children from
	*    @param WorldLocation world space location to find the closest child from
	*    @param Radius Radius to use from the WorldLocation
	*    @param bAlwaysReturnClosest if radius query does not return anything still return the closest from the point
	*/
	CHAOS_API TArray<FPBDRigidParticleHandle*> FindChildrenWithinRadius(const FPBDRigidClusteredParticleHandle* ClusteredParticle, const FVec3& WorldLocation, FReal Radius, bool bAlwaysReturnClosest) const;

	/*
	*  FindParticlesWithinRadius
	*    Find the closest particle from an array of  particle from a world position and a radius 
	*    current implementation will pick the closest based on the distance from the center
	*    future implementation may expose option for more precise queries
	*    if bAlwaysReturnClosest is checked this will always return  the closest from the location even if the radius does not encompass any center of mass
	*    @param Particles array of clustered particles to select from  
	*    @param WorldLocation world space location to find the closest particle from
	*    @param bAlwaysReturnClosest if radius query does not return anything still return the closest from the point
	*/
	static CHAOS_API TArray<FPBDRigidParticleHandle*> FindParticlesWithinRadius(const TArray<FPBDRigidParticleHandle*>& Particles, const FVec3& WorldLocation, FReal Radius, bool bAlwaysReturnClosest);
	
	/**
	* GenerateConnectionGraph
	*   Creates a connection graph for the given index using the creation parameters. This will not
	*   clear the existing graph.
	*/
	void SetClusterConnectionFactor(FReal ClusterConnectionFactorIn) { MClusterConnectionFactor = ClusterConnectionFactorIn; }
	void SetClusterUnionConnectionType(FClusterCreationParameters::EConnectionMethod ClusterConnectionType) { MClusterUnionConnectionType = ClusterConnectionType; }
	FClusterCreationParameters::EConnectionMethod GetClusterUnionConnectionType() const { return MClusterUnionConnectionType; }

	void ApplySettings(const FChaosSolverConfiguration& Settings);

	CHAOS_API void GenerateConnectionGraph(
		const TArray<FPBDRigidParticleHandle*>& Particles,
		const FClusterCreationParameters& Parameters = FClusterCreationParameters(),
		const TSet<FPBDRigidParticleHandle*>* FromParticles = nullptr,
		const TSet<FPBDRigidParticleHandle*>* ToParticles = nullptr);

	CHAOS_API void GenerateConnectionGraph(
		Chaos::FPBDRigidClusteredParticleHandle* Parent,
		const FClusterCreationParameters & Parameters = FClusterCreationParameters());

	CHAOS_API void ClearConnectionGraph(FPBDRigidClusteredParticleHandle* Parent);

	const TSet<Chaos::FPBDRigidClusteredParticleHandle*>& GetTopLevelClusterParents() const { return TopLevelClusterParents; }
	TSet<Chaos::FPBDRigidClusteredParticleHandle*>& GetTopLevelClusterParents() { return TopLevelClusterParents; }

	FRigidEvolution& GetEvolution() { return MEvolution; }
	const FRigidEvolution& GetEvolution() const { return MEvolution; }

	CHAOS_API void SetInternalStrain(FPBDRigidClusteredParticleHandle* Particle, FRealSingle Strain);
	CHAOS_API void SetExternalStrain(FPBDRigidClusteredParticleHandle* Particle, FRealSingle Strain);

	/*
	*  BuildConvexOptimizer
	*    Create the convex optimizer unique ptr and loop over the particle geometry to simplify
	*    all the convexes within the hierarchy
	*    @param Particle particle on which the geometry will be simplified   
	*/
	CHAOS_API void BuildConvexOptimizer(FPBDRigidClusteredParticleHandle* Particle);
	
	static CHAOS_API bool ShouldUnionsHaveCollisionParticles();

	FClusterUnionManager& GetClusterUnionManager() { return ClusterUnionManager; }
	const FClusterUnionManager& GetClusterUnionManager() const { return ClusterUnionManager; }

	UE_DEPRECATED(5.4, "No longer expose publicly - now return empty set")
	const TSet<Chaos::FPBDRigidClusteredParticleHandle*>& GetTopLevelClusterParentsStrained() const
	{ 
		static const TSet<Chaos::FPBDRigidClusteredParticleHandle*> ConstEmptySet;
		return ConstEmptySet;
	}

	// Remove connectivity edges for specified particles
	CHAOS_API void RemoveNodeConnections(FPBDRigidParticleHandle* Child);
	CHAOS_API void RemoveNodeConnections(FPBDRigidClusteredParticleHandle* Child);

	template<typename TFilter>
	void RemoveFilteredNodeConnections(FPBDRigidClusteredParticleHandle* ClusteredChild, TFilter&& Filter)
	{
		check(ClusteredChild != nullptr);

		constexpr bool bHasFilter = std::is_invocable_r_v < bool, TFilter, const TConnectivityEdge<FReal>&>;
		TArray<TConnectivityEdge<FReal>>& Edges = ClusteredChild->ConnectivityEdges();
		for (int32 EdgeIndex = Edges.Num() - 1; EdgeIndex >= 0; --EdgeIndex)
		{
			const TConnectivityEdge<FReal>& Edge = Edges[EdgeIndex];
			FPBDRigidParticleHandle* Sibling = Edge.Sibling;
			if constexpr (bHasFilter) 
			{
				if (!Filter(Edge))
				{
					continue;
				}

				Edges.RemoveAtSwap(EdgeIndex, EAllowShrinking::No);
			}

			check(Sibling != nullptr);
			TArray<TConnectivityEdge<FReal>>& OtherEdges = Sibling->CastToClustered()->ConnectivityEdges();
			const int32 Idx = OtherEdges.IndexOfByKey(ClusteredChild);
			if (Idx != INDEX_NONE)
			{
				OtherEdges.RemoveAtSwap(Idx);
			}

			// Make sure there are no duplicates!
			check(OtherEdges.IndexOfByKey(ClusteredChild) == INDEX_NONE);
		}

		if constexpr (!bHasFilter)
		{
			Edges.SetNum(0);
		}
		else
		{
			Edges.Shrink();
		}
	}

	template<typename ParticleHandleTypeA, typename ParticleHandleTypeB>
	void CreateNodeConnection(ParticleHandleTypeA* A, ParticleHandleTypeB* B)
	{
		if(A && B)
		{
			CreateNodeConnection(A->CastToClustered(), B->CastToClustered());
		}
		else
		{
			ensureMsgf(false, TEXT("CreateNodeConnection asked to connect a null particle, ignoring connection."));
		}
	}

	CHAOS_API void CreateNodeConnection(FPBDRigidClusteredParticleHandle* A, FPBDRigidClusteredParticleHandle* B);


	/**
	 * CleanupInternalClustersForProxies
	 *	For a given set of physics proxies, cleanup any tracked internal clusters that we've marked as being empty.
	 */
	CHAOS_API void CleanupInternalClustersForProxies(TArrayView<IPhysicsProxyBase*> Proxies);

	/**
	* Handles leveraging the connectivity edges on the children of the clustered particle to produce the desired effects.
	*/
	CHAOS_API TSet<FPBDRigidParticleHandle*> HandleConnectivityOnReleaseClusterParticle(FPBDRigidClusteredParticleHandle* ClusteredParticle, bool bCreateNewClusters);

	CHAOS_API void DisableCluster(FPBDRigidClusteredParticleHandle* ClusteredParticle);

	bool ShouldThrottleParticleRelease() const;
	void ThrottleReleasedParticlesIfNecessary(TSet<FPBDRigidParticleHandle*>& Particles);
	void ThrottleReleasedParticlesIfNecessary(TArray<FPBDRigidParticleHandle*>& Particles);

	/** disable a particle and make sure the related internal structure are up to date */
	void DisableParticle(FPBDRigidParticleHandle* ParticleToDisable, bool bRemoveFromChildrenMap = true);

	const FChaosSolverDestructionSettings& GetDestructionSettings() const { return DestructionSettings; }

 protected:

	CHAOS_API void ComputeStrainFromCollision(const FPBDCollisionConstraints& CollisionRule, const FReal Dt);
	CHAOS_API void ResetCollisionImpulseArray();
	CHAOS_API void ApplyStrainModifiers(const TArray<FPBDRigidClusteredParticleHandle*>& StrainedParticles);

	/*
	* Connectivity
	*/
	CHAOS_API void UpdateConnectivityGraphUsingPointImplicit(
		const TArray<FPBDRigidParticleHandle*>& Particles,
		FReal CollisionThicknessPercent,
		const TSet<FPBDRigidParticleHandle*>* FromParticles = nullptr,
		const TSet<FPBDRigidParticleHandle*>* ToParticles = nullptr);
	CHAOS_API void UpdateConnectivityGraphUsingPointImplicit(
		Chaos::FPBDRigidClusteredParticleHandle* Parent,
		const FClusterCreationParameters& Parameters = FClusterCreationParameters());

	CHAOS_API void FixConnectivityGraphUsingDelaunayTriangulation(
		const TArray<FPBDRigidParticleHandle*>& Particles,
		const FClusterCreationParameters& Parameters = FClusterCreationParameters(),
		const TSet<FPBDRigidParticleHandle*>* FromParticles = nullptr,
		const TSet<FPBDRigidParticleHandle*>* ToParticles = nullptr);
	CHAOS_API void FixConnectivityGraphUsingDelaunayTriangulation(
		Chaos::FPBDRigidClusteredParticleHandle* Parent,
		const FClusterCreationParameters& Parameters = FClusterCreationParameters());

	CHAOS_API void UpdateConnectivityGraphUsingDelaunayTriangulation(
		const TArray<FPBDRigidParticleHandle*>& Particles,
		const FClusterCreationParameters& Parameters = FClusterCreationParameters(),
		const TSet<FPBDRigidParticleHandle*>* FromParticles = nullptr,
		const TSet<FPBDRigidParticleHandle*>* ToParticles = nullptr);
	CHAOS_API void UpdateConnectivityGraphUsingDelaunayTriangulation(
		const Chaos::FPBDRigidClusteredParticleHandle* Parent,
		const FClusterCreationParameters& Parameters = FClusterCreationParameters());

	CHAOS_API void UpdateConnectivityGraphUsingDelaunayTriangulationWithBoundsOverlaps(
		const TArray<FPBDRigidParticleHandle*>& Particles,
		const FClusterCreationParameters& Parameters = FClusterCreationParameters(),
		const TSet<FPBDRigidParticleHandle*>* FromParticles = nullptr,
		const TSet<FPBDRigidParticleHandle*>* ToParticles = nullptr);
	CHAOS_API void UpdateConnectivityGraphUsingDelaunayTriangulationWithBoundsOverlaps(
		const Chaos::FPBDRigidClusteredParticleHandle* Parent,
		const FClusterCreationParameters& Parameters = FClusterCreationParameters());
	
	CHAOS_API void RemoveChildFromParent(FPBDRigidParticleHandle* Child, FPBDRigidClusteredParticleHandle* ClusteredParent);
	CHAOS_API void RemoveChildFromParentAndChildrenArray(FPBDRigidParticleHandle* Child, FPBDRigidClusteredParticleHandle* ClusteredParent);

	// When a body has broken due to contact resolution, record an entry in a set
	// for the collision and the particle who's momentum should be restored.
	CHAOS_API void TrackBreakingCollision(FPBDRigidClusteredParticleHandle* ClusteredParticle);

	// Restore some percentage of momenta for objects which were involved in collisions
	// with destroyed GCs
	CHAOS_API void RestoreBreakingMomentum();
	CHAOS_API void RemoveFromMomentumRestoringStructures(const FPBDRigidParticleHandle* ParticleToRemove);

	CHAOS_API void SendBreakingEvent(FPBDRigidClusteredParticleHandle* ClusteredParticle, bool bFromCrumble);
	CHAOS_API void SendCrumblingEvent(FPBDRigidClusteredParticleHandle* ClusteredParticle);

	CHAOS_API TSet<FPBDRigidParticleHandle*> ReleaseClusterParticlesImpl(
		FPBDRigidClusteredParticleHandle* ClusteredParticle, 
		bool bForceRelease,
		bool bCreateNewClusters);

	using FParticleIsland = TArray<FPBDRigidParticleHandle*>;
	CHAOS_API TArray<FParticleIsland> FindIslandsInChildren(const FPBDRigidClusteredParticleHandle* ClusteredParticle, bool bTraverseInterclusterEdges);
	CHAOS_API TArray<FPBDRigidParticleHandle*> CreateClustersFromNewIslands(TArray<FParticleIsland>& Islands, FPBDRigidClusteredParticleHandle* ClusteredParent);

	CHAOS_API void UpdateTopLevelParticle(FPBDRigidClusteredParticleHandle* Particle);

	/**
	 * This function is a bit more versatile than the name suggests. This function can either be used to update the cluster properties
	 * incrementally or entirely rebuild the properties all over again. This all depends on whether the input children is
	 * either 1) the new children or 2) all the children as well as what those initial properties are set to.
	 */
	UE_DEPRECATED(5.4, "This should be handled for you properly in AddParticlesToCluster and RemoveParticlesFromCluster. There is no need for an extra function call.")
	CHAOS_API void UpdateClusterParticlePropertiesFromChildren(
		FPBDRigidClusteredParticleHandle* Cluster,
		const FRigidHandleArray& Children,
		const TMap<FPBDRigidParticleHandle*, FPBDRigidParticleHandle*>& ChildToParentMap);

private:

	// Cluster release stats for debugging with CVar p.Chaos.Clustering.DumpClusterAndReleaseStats
	uint32 AdvanceCount = 0;
	uint32 TotalProcessedClusters = 0;
	uint32 TotalReleasedChildren = 0;
	uint32 FrameProcessedClusters = 0;
	uint32 FrameReleasedChildren = 0;

	FRigidEvolution& MEvolution;
	FPBDRigidClusteredParticles& MParticles;
	TSet<Chaos::FPBDRigidClusteredParticleHandle*> TopLevelClusterParents;

	TMap<Chaos::FPBDRigidClusteredParticleHandle*, int64> TopLevelClusterParentsStrained;

	// Cluster data
	FClusterMap MChildren;

	/**
	 * The old cluster union map has been replaced by the cluster union manager to allow for more
	 * dynamic behavior of adding and removing particles from a cluster instead of being just restricted
	 * to unioning particles together at construction.
	 */
	FClusterUnionManager ClusterUnionManager;

	// Collision Impulses
	bool MCollisionImpulseArrayDirty;
	
	// Breaking data
	bool DoGenerateBreakingData;
	TArray<FBreakingData> MAllClusterBreakings;

	TArray<FCrumblingData> MAllClusterCrumblings;

	TSet<FPBDRigidClusteredParticleHandle*> CrumbledSinceLastUpdate;
	TMap<IPhysicsProxyBase*, TArray<FPBDRigidClusteredParticleHandle*>> EmptyInternalClustersPerProxy;

	struct FMomentumRestoringData
	{
		FVector V = FVector::ZeroVector;
		FVector W = FVector::ZeroVector;
	};
	// this map stores the momentum information a particle had when colliding with a breaking clustered particle
	// this allow the colliding particle to keep going if the clustered particle break depsite the fact that the collision constraint resolution stopped it 
	TMap<FPBDRigidParticleHandle*, FMomentumRestoringData> MomentumRestoringDataByParticle;

	FReal MClusterConnectionFactor;
	FClusterCreationParameters::EConnectionMethod MClusterUnionConnectionType;

	// Sim callback objects which implement cluster modification steps
	const TArray<ISimCallbackObject*>* StrainModifiers;

	FChaosSolverDestructionSettings DestructionSettings;
};

} // namespace Chaos
