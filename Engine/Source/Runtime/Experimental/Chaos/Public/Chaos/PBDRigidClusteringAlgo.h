// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/PBDRigidClustering.h"

namespace Chaos
{
	enum class EMassOffsetType : uint8
	{
		None = 0,
		Position = 1 << 0,
		Rotation = 1 << 1
	};
	ENUM_CLASS_FLAGS(EMassOffsetType)

	enum class EInertiaOperations : uint8
	{
		None = 0,
		LocalInertiaDropOffDiagonalTerms = 1 << 0
	};
	ENUM_CLASS_FLAGS(EInertiaOperations)

	void CHAOS_API UpdateClusterMassProperties(
		FPBDRigidClusteredParticleHandle* Parent,
		const TSet<FPBDRigidParticleHandle*>& Children);

	// MassOffsetOperations & LocalInertiaDropOffDiagonalTerms: Discard off-diagonal inertia terms in particle's local space
	void CHAOS_API AdjustClusterInertia(FPBDRigidClusteredParticleHandle* Cluster, EInertiaOperations InertiaOperations);

	// MassOffsetTypes & Position: set X/P of Cluster to its world CoM, and set its CoM to ZeroVector
	// MassOffsetTypes & Rotation: set R/Q of Cluster to its world RoM, and set its RoM to Identity
	FRigidTransform3 CHAOS_API MoveClusterToMassOffset(FPBDRigidClusteredParticleHandle* Cluster, EMassOffsetType MassOffsetTypes);

	void CHAOS_API UpdateKinematicProperties(
		FPBDRigidParticleHandle* Parent,
		const FRigidClustering::FClusterMap&,
		FRigidClustering::FRigidEvolution&);

	void CHAOS_API UpdateGeometry(
		Chaos::FPBDRigidClusteredParticleHandle* Parent,
		const TSet<FPBDRigidParticleHandle*>& Children,
		const FRigidClustering::FClusterMap& ChildrenMap,
		const Chaos::FImplicitObjectPtr& ProxyGeometry,
		const FClusterCreationParameters& Parameters);
	
	// Scale proxy geometry and set the result to the particle handle
	void CHAOS_API BuildScaledGeometry(Chaos::FPBDRigidClusteredParticleHandle* ParticleHandle, const Chaos::FImplicitObjectPtr& ProxyGeometry, const FVector& WorldScale);

	// Update the collision acceleration structure and flags
	void CHAOS_API UpdateCollisionFlags(Chaos::FPBDRigidClusteredParticleHandle* ParticleHandle, const bool bUseParticleImplicit);

	UE_DEPRECATED(5.4, "Use UpdateGeometry with FImplicitObjectPtr instead")
	FORCEINLINE void CHAOS_API UpdateGeometry(
		Chaos::FPBDRigidClusteredParticleHandle* Parent,
		const TSet<FPBDRigidParticleHandle*>& Children,
		const FRigidClustering::FClusterMap& ChildrenMap,
		TSharedPtr<Chaos::FImplicitObject, ESPMode::ThreadSafe> ProxyGeometry,
		const FClusterCreationParameters& Parameters) {check(false);}


	/**
	* Update shapes with first valid filter data from a child. Should be called after a child's filter
	* is modified to ensure cluster's filters match at least one of the children and isn't stale.
	*/
	void UpdateClusterFilterDataFromChildren(FPBDRigidClusteredParticleHandle* ClusterParent, const TArray<FPBDRigidParticleHandle*>& Children);
} // namespace Chaos
