// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/PBDRigidClustering.h"

namespace Chaos
{

inline TArray<FVec3> CleanCollisionParticles(
	const TArray<FVec3>& Vertices,
	FAABB3 BBox, 
	const FReal SnapDistance=(FReal)0.01)
{
	const int32 NumPoints = Vertices.Num();
	if (NumPoints <= 1)
		return TArray<FVec3>(Vertices);

	FReal MaxBBoxDim = BBox.Extents().Max();
	if (MaxBBoxDim < SnapDistance)
		return TArray<FVec3>(&Vertices[0], 1);

	BBox.Thicken(FMath::Max(SnapDistance/10, (FReal)(UE_KINDA_SMALL_NUMBER*10))); // 0.001
	MaxBBoxDim = BBox.Extents().Max();

	const FVec3 PointsCenter = BBox.Center();
	TArray<FVec3> Points(Vertices);

	// Find coincident vertices.  We hash to a grid of fine enough resolution such
	// that if 2 particles hash to the same cell, then we're going to consider them
	// coincident.
	TSet<int64> OccupiedCells;
	OccupiedCells.Reserve(NumPoints);

	TArray<int32> Redundant;
	Redundant.Reserve(NumPoints); // Excessive, but ensures consistent performance.

	int32 NumCoincident = 0;
	const int64 Resolution = static_cast<int64>(floor(MaxBBoxDim / FMath::Max(SnapDistance,(FReal)UE_KINDA_SMALL_NUMBER)));
	const FReal CellSize = static_cast<FReal>(static_cast<double>(MaxBBoxDim) / static_cast<double>(Resolution));
	for (int32 i = 0; i < 2; i++)
	{
		Redundant.Reset();
		OccupiedCells.Reset();
		// Shift the grid by 1/2 a grid cell the second iteration so that
		// we don't miss slightly adjacent coincident points across cell
		// boundaries.
		const FVec3 GridCenter = FVec3(0) - FVec3(static_cast<FReal>(i) * CellSize / 2);
		for (int32 j = 0; j < Points.Num(); j++)
		{
			const FVec3 Pos = Points[j] - PointsCenter; // Centered at the origin
			const TVec3<int64> Coord(
				static_cast<int64>(FMath::Floor((Pos[0] - GridCenter[0]) / CellSize + static_cast<double>(Resolution) / 2)),
				static_cast<int64>(FMath::Floor((Pos[1] - GridCenter[1]) / CellSize + static_cast<double>(Resolution) / 2)),
				static_cast<int64>(FMath::Floor((Pos[2] - GridCenter[2]) / CellSize + static_cast<double>(Resolution) / 2)));
			const int64 FlatIdx =
				((Coord[0] * Resolution + Coord[1]) * Resolution) + Coord[2];

			bool AlreadyInSet = false;
			OccupiedCells.Add(FlatIdx, &AlreadyInSet);
			if (AlreadyInSet)
				Redundant.Add(j);
		}

		for (int32 j = Redundant.Num(); j--;)
		{
			Points.RemoveAt(Redundant[j]);
		}
	}

	// Shrink the array, if appropriate
	Points.SetNum(Points.Num(), EAllowShrinking::Yes);
	return Points;
}

inline TArray<FVec3> CleanCollisionParticles(
	const TArray<FVec3>& Vertices, 
	const FReal SnapDistance=(FReal)0.01)
{
	if (!Vertices.Num())
	{
		return TArray<FVec3>();
	}
	FAABB3 BBox(FAABB3::EmptyAABB());
	for (const FVec3& Pt : Vertices)
	{
		BBox.GrowToInclude(Pt);
	}
	return CleanCollisionParticles(Vertices, BBox, SnapDistance);
}

inline TArray<FVec3> CleanCollisionParticles(
	FTriangleMesh &TriMesh, 
	const TArrayView<const FVec3>& Vertices, 
	const FReal Fraction)
{
	TArray<FVec3> CollisionVertices;
	if (Fraction <= 0.0)
		return CollisionVertices;

	// If the tri mesh has any open boundaries, see if we can merge any coincident
	// vertices on the boundary.  This makes the importance ordering work much better
	// as we need the curvature at each edge of the tri mesh, and we can't calculate
	// curvature on discontiguous triangles.
	TSet<int32> BoundaryPoints = TriMesh.GetBoundaryPoints();
	if (BoundaryPoints.Num())
	{
		TMap<int32, int32> Remapping =
			TriMesh.FindCoincidentVertexRemappings(BoundaryPoints.Array(), Vertices);
		TriMesh.RemapVertices(Remapping);
	}

	// Get the importance vertex ordering, from most to least.  Reorder the 
	// particles accordingly.
	TArray<int32> CoincidentVertices;
	const TArray<int32> Ordering = TriMesh.GetVertexImportanceOrdering(Vertices, &CoincidentVertices, true);

	// Particles are ordered from most important to least, with coincident 
	// vertices at the very end.
	const int32 NumGoodPoints = Ordering.Num() - CoincidentVertices.Num();

#if DO_GUARD_SLOW
	for (int i = NumGoodPoints; i < Ordering.Num(); ++i)
	{
		ensure(CoincidentVertices.Contains(Ordering[i]));	//make sure all coincident vertices are at the back
	}
#endif

	CollisionVertices.AddUninitialized(FMath::Min(NumGoodPoints, static_cast<int32>(ceil(static_cast<FReal>(NumGoodPoints) * Fraction))));
	for (int i = 0; i < CollisionVertices.Num(); i++)
	{
		CollisionVertices[i] = Vertices[Ordering[i]];
	}
	return CollisionVertices;
}

inline void CleanCollisionParticles(
	FTriangleMesh &TriMesh, 
	const TArrayView<const FVec3>& Vertices, 
	const FReal Fraction,
	TSet<int32>& ResultingIndices)
{
	ResultingIndices.Reset();
	if (Fraction <= 0.0)
		return;

	TArray<int32> CoincidentVertices;
	const TArray<int32> Ordering = TriMesh.GetVertexImportanceOrdering(Vertices, &CoincidentVertices, true);
	int32 NumGoodPoints = Ordering.Num() - CoincidentVertices.Num();
	NumGoodPoints = FMath::Min(NumGoodPoints, static_cast<int32>(ceil(static_cast<FReal>(NumGoodPoints) * Fraction)));

	ResultingIndices.Reserve(NumGoodPoints);
	for (int32 i = 0; i < NumGoodPoints; i++)
	{
		ResultingIndices.Add(Ordering[i]);
	}
}

} // namespace Chaos
