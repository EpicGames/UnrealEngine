// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HAL/Platform.h"

namespace Chaos
{
	enum class EDamageEvaluationModel: uint8
	{
		/** 
		* particles internal strains are set from user defined damage thresholds 
		* It's a simple model, fast to evaluate , but can be set to be non realistic
		* and may require more fine tuning 
		*/
		StrainFromDamageThreshold,

		/**
		* Particle internal strains are set from the strength properties of the physics material
		* and the surface area of connection of a piece to another 
		* this is as fast as the damage threshold to evaluate but provide a more realistic model 
		* and allow for simpler tuning based on material
		*/
		StrainFromMaterialStrengthAndConnectivity,
	};

} // namespace Chaos
