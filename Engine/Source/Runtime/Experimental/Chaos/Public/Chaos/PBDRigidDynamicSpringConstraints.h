// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Array.h"
#include "Chaos/Evolution/IndexedConstraintContainer.h"
#include "Chaos/ParticleHandle.h"

namespace Chaos
{
	class FPBDRigidDynamicSpringConstraints;

	class FPBDRigidDynamicSpringConstraintHandle final : public TIndexedContainerConstraintHandle<FPBDRigidDynamicSpringConstraints>
	{
	public:
		using Base = TIndexedContainerConstraintHandle<FPBDRigidDynamicSpringConstraints>;
		using FConstraintContainer = FPBDRigidDynamicSpringConstraints;

		FPBDRigidDynamicSpringConstraintHandle() 
		{
		}

		FPBDRigidDynamicSpringConstraintHandle(FConstraintContainer* InConstraintContainer, int32 InConstraintIndex) 
			: TIndexedContainerConstraintHandle<FPBDRigidDynamicSpringConstraints>(InConstraintContainer, InConstraintIndex)
		{
		}

		CHAOS_API virtual FParticlePair GetConstrainedParticles() const override final;

		static const FConstraintHandleTypeID& StaticType()
		{
			static FConstraintHandleTypeID STypeID(TEXT("FRigidDynamicSpringConstraintHandle"), &FIndexedConstraintHandle::StaticType());
			return STypeID;
		}

	protected:
		using Base::ConstraintIndex;
		using Base::ConcreteContainer;
	};

	class FPBDRigidDynamicSpringConstraints : public TPBDIndexedConstraintContainer<FPBDRigidDynamicSpringConstraints>
	{
	public:
		using Base = TPBDIndexedConstraintContainer<FPBDRigidDynamicSpringConstraints>;
		using FConstrainedParticlePair = TVec2<FGeometryParticleHandle*>;
		using FConstraintContainerHandle = FPBDRigidDynamicSpringConstraintHandle;
		using FConstraintHandleAllocator = TConstraintHandleAllocator<FPBDRigidDynamicSpringConstraints>;
		using FHandles = TArray<FConstraintContainerHandle*>;

		FPBDRigidDynamicSpringConstraints(const FReal InStiffness = (FReal)1.)
			: TPBDIndexedConstraintContainer<FPBDRigidDynamicSpringConstraints>(FConstraintContainerHandle::StaticType())
			, CreationThreshold(1)
			, MaxSprings(1)
			, Stiffness(InStiffness) 
		{}

		FPBDRigidDynamicSpringConstraints(TArray<FConstrainedParticlePair>&& InConstraints, const FReal InCreationThreshold = (FReal)1., const int32 InMaxSprings = 1, const FReal InStiffness = (FReal)1.)
			: TPBDIndexedConstraintContainer<FPBDRigidDynamicSpringConstraints>(FConstraintContainerHandle::StaticType())
			, Constraints(MoveTemp(InConstraints))
			, CreationThreshold(InCreationThreshold)
			, MaxSprings(InMaxSprings)
			, Stiffness(InStiffness)
		{
			if (Constraints.Num() > 0)
			{
				Handles.Reserve(Constraints.Num());
				Distances.Reserve(Constraints.Num());
				SpringDistances.Reserve(Constraints.Num());
				ConstraintSolverBodies.Reserve(Constraints.Num());
				for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
				{
					Handles.Add(HandleAllocator.AllocHandle(this, ConstraintIndex));
					Distances.Add({});
					SpringDistances.Add({});
					ConstraintSolverBodies.Add({ nullptr, nullptr });
				}
			}
		}

		virtual ~FPBDRigidDynamicSpringConstraints() = default;

		//
		// Constraint Container API
		//

		/**
		 * Get the number of constraints.
		 */
		int32 NumConstraints() const
		{
			return Constraints.Num();
		}

		/**
		 * Add a constraint initialized from current world-space particle positions.
		 * You would use this method when your objects are already positioned in the world.
		 */
		FConstraintContainerHandle* AddConstraint(const FConstrainedParticlePair& InConstrainedParticles)
		{
			Handles.Add(HandleAllocator.AllocHandle(this, Handles.Num()));
			Constraints.Add(InConstrainedParticles);
			Distances.Add({});
			SpringDistances.Add({});
			ConstraintSolverBodies.Add({ nullptr, nullptr });
			return Handles.Last();
		}

		/**
		 * Remove the specified constraint.
		 */
		void RemoveConstraint(int ConstraintIndex)
		{
			FConstraintContainerHandle* ConstraintHandle = Handles[ConstraintIndex];
			if (ConstraintHandle != nullptr)
			{
				// Release the handle for the freed constraint
				HandleAllocator.FreeHandle(ConstraintHandle);
				Handles[ConstraintIndex] = nullptr;
			}

			// Swap the last constraint into the gap to keep the array packed
			Constraints.RemoveAtSwap(ConstraintIndex);
			Distances.RemoveAtSwap(ConstraintIndex);
			SpringDistances.RemoveAtSwap(ConstraintIndex);
			ConstraintSolverBodies.RemoveAtSwap(ConstraintIndex);
			Handles.RemoveAtSwap(ConstraintIndex);

			// Update the handle for the constraint that was moved
			if (ConstraintIndex < Handles.Num())
			{
				SetConstraintIndex(Handles[ConstraintIndex], ConstraintIndex);
			}
		}

		/**
		 * Disabled the specified constraint.
		 */
		void DisableConstraints(const TSet<FGeometryParticleHandle*>& RemovedParticles)
		{
			// @todo(chaos)
		}


		/**
		 * Set the distance threshold below which springs get created between particles.
		 */
		void SetCreationThreshold(const FReal InCreationThreshold)
		{
			CreationThreshold = InCreationThreshold;
		}

		/**
		 * Set the maximum number of springs
		 */
		void SetMaxSprings(const int32 InMaxSprings)
		{
			MaxSprings = InMaxSprings;
		}


		//
		// Constraint API
		//
		FHandles& GetConstraintHandles()
		{
			return Handles;
		}
		const FHandles& GetConstConstraintHandles() const
		{
			return Handles;
		}


		const FConstraintContainerHandle* GetConstraintHandle(int32 ConstraintIndex) const
		{
			return Handles[ConstraintIndex];
		}

		FConstraintContainerHandle* GetConstraintHandle(int32 ConstraintIndex)
		{
			return Handles[ConstraintIndex];
		}


		/**
		 * Get the particles that are affected by the specified constraint.
		 */
		const FConstrainedParticlePair& GetConstrainedParticles(int32 ConstraintIndex) const
		{
			return Constraints[ConstraintIndex];
		}

		CHAOS_API void UpdatePositionBasedState(const FReal Dt);

		//
		// FConstraintContainer Implementation
		//
		virtual int32 GetNumConstraints() const override final { return NumConstraints(); }
		virtual void ResetConstraints() override final {}
		CHAOS_API virtual void AddConstraintsToGraph(Private::FPBDIslandManager& IslandManager) override final;
		virtual void PrepareTick() override final {}
		virtual void UnprepareTick() override final {}

		//
		// TSimpleConstraintContainerSolver API - used by RBAN solvers
		//
		CHAOS_API void AddBodies(FSolverBodyContainer& SolverBodyContainer);
		void GatherInput(const FReal Dt) {}
		CHAOS_API void ScatterOutput(const FReal Dt);
		CHAOS_API void ApplyPositionConstraints(const FReal Dt, const int32 It, const int32 NumIts);
		void ApplyVelocityConstraints(const FReal Dt, const int32 It, const int32 NumIts) {}
		void ApplyProjectionConstraints(const FReal Dt, const int32 It, const int32 NumIts) {}

		//
		// TIndexedConstraintContainerSolver API - used by World solvers
		//
		CHAOS_API void AddBodies(const TArrayView<int32>& ConstraintIndices, FSolverBodyContainer& SolverBodyContainer);
		void GatherInput(const TArrayView<int32>& ConstraintIndices, const FReal Dt) {}
		CHAOS_API void ScatterOutput(const TArrayView<int32>& ConstraintIndices, const FReal Dt);
		CHAOS_API void ApplyPositionConstraints(const TArrayView<int32>& ConstraintIndices, const FReal Dt, const int32 It, const int32 NumIts);
		void ApplyVelocityConstraints(const TArrayView<int32>& ConstraintIndices, const FReal Dt, const int32 It, const int32 NumIts) {}
		void ApplyProjectionConstraints(const TArrayView<int32>& ConstraintIndices, const FReal Dt, const int32 It, const int32 NumIts) {}

	protected:
		using Base::GetConstraintIndex;
		using Base::SetConstraintIndex;

	private:
		CHAOS_API void AddBodies(const int32 ConstraintIndex, FSolverBodyContainer& SolverBodyContainer);
		CHAOS_API void ApplySingle(const FReal Dt, int32 ConstraintIndex) const;

		CHAOS_API FVec3 GetDelta(const FVec3& WorldSpaceX1, const FVec3& WorldSpaceX2, const int32 ConstraintIndex, const int32 SpringIndex) const;

		TArray<FConstrainedParticlePair> Constraints;
		TArray<TArray<TVec2<FVec3>>> Distances;
		TArray<TArray<FReal>> SpringDistances;
		FReal CreationThreshold;
		int32 MaxSprings;
		FReal Stiffness;

		TArray<FSolverBodyPtrPair> ConstraintSolverBodies;

		FHandles Handles;
		FConstraintHandleAllocator HandleAllocator;
	};

	template<class T, int d>
	using TPBDRigidDynamicSpringConstraintHandle UE_DEPRECATED(4.27, "Deprecated. this class is to be deleted, use FPBDRigidDynamicSpringConstraintHandle instead") = FPBDRigidDynamicSpringConstraintHandle;

	template<class T, int d>
	using TPBDRigidDynamicSpringConstraints UE_DEPRECATED(4.27, "Deprecated. this class is to be deleted, use FPBDRigidDynamicSpringConstraints instead") = FPBDRigidDynamicSpringConstraints;
}
