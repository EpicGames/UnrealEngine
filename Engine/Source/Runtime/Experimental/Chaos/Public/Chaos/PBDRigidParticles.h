// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/ArrayCollectionArray.h"
#include "Chaos/RigidParticles.h"
#include "Chaos/Rotation.h"

#if defined(_MSC_VER)
#pragma warning(push)
#pragma warning(disable:4946)
#endif

namespace Chaos
{
template<class T, int d>
class TPBDRigidsEvolution;

CHAOS_API void EnsureSleepingObjectState(EObjectStateType ObjectState);

template<class T, int d>
class TPBDRigidParticles : public TRigidParticles<T, d>
{
	friend class TPBDRigidsEvolution<T, d>;

  public:
	using TRigidParticles<T, d>::CenterOfMass;
	using TRigidParticles<T, d>::RotationOfMass;
	using TRigidParticles<T, d>::Sleeping;

	TPBDRigidParticles()
	    : TRigidParticles<T, d>()
	{
		this->MParticleType = EParticleType::Rigid;
		RegisterArrays();
	}
	TPBDRigidParticles(const TPBDRigidParticles<T, d>& Other) = delete;
	TPBDRigidParticles(TPBDRigidParticles<T, d>&& Other)
	    : TRigidParticles<T, d>(MoveTemp(Other))
		, MP(MoveTemp(Other.MP))
		, MQ(MoveTemp(Other.MQ))
		, MPreV(MoveTemp(Other.MPreV))
		, MPreW(MoveTemp(Other.MPreW))
		, MSolverBodyIndex(MoveTemp(Other.MSolverBodyIndex))
	{
		this->MParticleType = EParticleType::Rigid;
		RegisterArrays();
	}

	virtual ~TPBDRigidParticles() = default;

	void RegisterArrays()
	{
		TArrayCollection::AddArray(&MP);
		TArrayCollection::AddArray(&MQ);
		TArrayCollection::AddArray(&MPreV);
		TArrayCollection::AddArray(&MPreW);
		TArrayCollection::AddArray(&MSolverBodyIndex);
	}

	UE_DEPRECATED(5.4, "Use GetP instead")
	FORCEINLINE const TVector<T, d>& P(const int32 index) const { return MP[index]; }
	UE_DEPRECATED(5.4, "Use GetP or SetP instead")
	FORCEINLINE TVector<T, d>& P(const int32 index) { return MP[index]; }
	FORCEINLINE const TVector<T, d>& GetP(const int32 index) const { return MP[index]; }
	FORCEINLINE void SetP(const int32 index, const TVector<T, d>& InP) { MP[index] = InP; }

	UE_DEPRECATED(5.4, "Use GetQ instead")
	FORCEINLINE const TRotation<T, d> Q(const int32 index) const { return TRotation<T, d>(MQ[index]); }
	UE_DEPRECATED(5.4, "Use GetQ or SetQ instead")
	FORCEINLINE TRotation<T, d> Q(const int32 index) { return TRotation<T, d>(MQ[index]); }
	FORCEINLINE const TRotation<T, d> GetQ(const int32 index) const { return TRotation<T, d>(MQ[index]); }
	FORCEINLINE void SetQ(const int32 index, const TRotation<T, d>& InQ) { MQ[index] = TRotation<FRealSingle, d>(InQ); }
	FORCEINLINE const TRotation<FRealSingle, d> GetQf(const int32 index) const { return MQ[index]; }
	FORCEINLINE void SetQf(const int32 index, const TRotation<FRealSingle, d>& InQ) { MQ[index] = InQ; }

	UE_DEPRECATED(5.4, "Use GetPreV instead")
	const TVector<T, d> PreV(const int32 index) const { return TVector<T, d>(MPreV[index]); }
	UE_DEPRECATED(5.4, "Use GetPreV or SetPreV instead")
	TVector<T, d> PreV(const int32 index) { return TVector<T, d>(MPreV[index]); }
	const TVector<T, d> GetPreV(const int32 index) const { return TVector<T, d>(MPreV[index]); }
	void SetPreV(const int32 index, const TVector<T, d>& InPreV) { MPreV[index] = TVector<FRealSingle, d>(InPreV); }
	const TVector<FRealSingle, d> GetPreVf(const int32 index) const { return MPreV[index]; }
	void SetPreVf(const int32 index, const TVector<FRealSingle, d>& InPreV) { MPreV[index] = InPreV; }

	UE_DEPRECATED(5.4, "Use GetPreW instead")
	const TVector<T, d> PreW(const int32 index) const { return MPreW[index]; }
	UE_DEPRECATED(5.4, "Use GetPreW or SetPreW instead")
	TVector<T, d> PreW(const int32 index) { return MPreW[index]; }
	const TVector<T, d> GetPreW(const int32 index) const { return MPreW[index]; }
	void SetPreW(const int32 index, const TVector<T, d>& InPreW) { MPreW[index] = TVector<FRealSingle, d>(InPreW); }
	const TVector<FRealSingle, d> GetPreWf(const int32 index) const { return MPreW[index]; }
	void SetPreWf(const int32 index, const TVector<FRealSingle, d>& InPreW) { MPreW[index] = InPreW; }

	// World-space center of mass location
	const TVector<T, d> XCom(const int32 index) const { return this->GetX(index) + this->GetR(index).RotateVector(CenterOfMass(index)); }
	const TVector<T, d> PCom(const int32 index) const { return this->GetP(index) + this->GetQ(index).RotateVector(CenterOfMass(index)); }

	// World-space center of mass rotation
	const TRotation<T, d> RCom(const int32 index) const { return this->GetR(index) * RotationOfMass(index); }
	const TRotation<T, d> QCom(const int32 index) const { return this->GetQ(index) * RotationOfMass(index); }

	void SetTransformPQCom(const int32 index, const TVector<T, d>& InPCom, const TRotation<T, d>& InQCom)
	{
		SetQ(index, InQCom * RotationOfMass(index).Inverse());
		SetP(index, InPCom - GetQ(index) * CenterOfMass(index));
	}

	// The index into an FSolverBodyContainer (for dynamic particles only), or INDEX_NONE.
	// \see FSolverBodyContainer
	int32 SolverBodyIndex(const int32 index) const { return MSolverBodyIndex[index]; }
	void SetSolverBodyIndex(const int32 index, const int32 InSolverBodyIndex) { MSolverBodyIndex[index] = InSolverBodyIndex; }

    // Must be reinterpret cast instead of static_cast as it's a forward declare
	typedef TPBDRigidParticleHandle<T, d> THandleType;
	const THandleType* Handle(int32 Index) const { return reinterpret_cast<const THandleType*>(TGeometryParticles<T,d>::Handle(Index)); }

	//cannot be reference because double pointer would allow for badness, but still useful to have non const access to handle
	THandleType* Handle(int32 Index) { return reinterpret_cast<THandleType*>(TGeometryParticles<T, d>::Handle(Index)); }

	void SetSleeping(int32 Index, bool bSleeping)
	{
		if (Sleeping(Index) && bSleeping == false)
		{
			SetPreV(Index, this->GetV(Index));
			SetPreW(Index, this->GetW(Index));
		}
		else if (bSleeping)
		{
			//being put to sleep, so zero out velocities
			this->SetV(Index, FVec3(0));
			this->SetW(Index, FVec3(0));
		}

		bool CurrentlySleeping = this->ObjectState(Index) == EObjectStateType::Sleeping;
		if (CurrentlySleeping != bSleeping)
		{
			TGeometryParticleHandle<T, d>* Particle = reinterpret_cast<TGeometryParticleHandle<T, d>*>(this->Handle(Index));
			this->AddSleepData(Particle, bSleeping);
		}

		// Dynamic -> Sleeping or Sleeping -> Dynamic
		if (this->ObjectState(Index) == EObjectStateType::Dynamic || this->ObjectState(Index) == EObjectStateType::Sleeping)
		{
			this->ObjectState(Index) = bSleeping ? EObjectStateType::Sleeping : EObjectStateType::Dynamic;
		}

		// Possible for code to set a Dynamic to Kinematic State then to Sleeping State
		if (this->ObjectState(Index) == EObjectStateType::Kinematic && bSleeping)
		{
			this->ObjectState(Index) =  EObjectStateType::Sleeping;
		}

		if (bSleeping)
		{
			EnsureSleepingObjectState(this->ObjectState(Index));
		}
	}

	void SetObjectState(int32 Index, EObjectStateType InObjectState)
	{
		const EObjectStateType CurrentState = this->ObjectState(Index);

		if (CurrentState == EObjectStateType::Uninitialized)
		{
			// When the state is first initialized, treat it like a static.
			this->InvM(Index) = 0.0f;
			this->InvI(Index) = TVec3<FRealSingle>(0);
		}

		if ((CurrentState == EObjectStateType::Dynamic || CurrentState == EObjectStateType::Sleeping) && (InObjectState == EObjectStateType::Kinematic || InObjectState == EObjectStateType::Static))
		{
			// Transitioning from dynamic to static or kinematic, set inverse mass and inertia tensor to zero.
			this->InvM(Index) = 0.0f;
			this->InvI(Index) = TVec3<FRealSingle>(0);
		}
		else if ((CurrentState == EObjectStateType::Kinematic || CurrentState == EObjectStateType::Static || CurrentState == EObjectStateType::Uninitialized) && (InObjectState == EObjectStateType::Dynamic || InObjectState == EObjectStateType::Sleeping))
		{
			// Transitioning from kinematic or static to dynamic, compute the inverses.
			this->InvM(Index) = FMath::IsNearlyZero(this->M(Index)) ? 0.0f : 1.f / this->M(Index);
			this->InvI(Index) = (this->I(Index).IsNearlyZero() || FMath::IsNearlyZero(this->M(Index))) ? TVec3<FRealSingle>::ZeroVector :
				TVec3<FRealSingle>(
				1.f / this->I(Index)[0],
				1.f / this->I(Index)[1],
				1.f / this->I(Index)[2]);

			this->SetP(Index, this->GetX(Index));
			this->SetQf(Index, this->GetRf(Index));
		}
		else if (InObjectState == EObjectStateType::Sleeping)
		{
			SetSleeping(Index, true);
			return;
		}
		
		const bool bCurrentSleeping = this->ObjectState(Index) == EObjectStateType::Sleeping;
		const bool bNewSleeping = InObjectState == EObjectStateType::Sleeping;
		if(bCurrentSleeping != bNewSleeping)
		{
			TGeometryParticleHandle<T, d>* Particle = reinterpret_cast<TGeometryParticleHandle<T, d>*>(this->Handle(Index));
 			this->AddSleepData(Particle, bNewSleeping);
		}

		this->ObjectState(Index) = InObjectState;
	}

	void SetSleepType(int32 Index, ESleepType InSleepType)
	{
		if (InSleepType == ESleepType::NeverSleep && this->ObjectState(Index) == EObjectStateType::Sleeping)
		{
			SetObjectState(Index, EObjectStateType::Dynamic);
		}

		this->SleepType(Index) = InSleepType;
	}

	FString ToString(int32 index) const
	{
		FString BaseString = TRigidParticles<T, d>::ToString(index);
		return FString::Printf(TEXT("%s, MP:%s, MQ:%s, MPreV:%s, MPreW:%s"), *BaseString, *GetP(index).ToString(), *GetQ(index).ToString(), *GetPreV(index).ToString(), *GetPreW(index).ToString());
	}

	virtual void Serialize(FChaosArchive& Ar) override
	{
		TRigidParticles<T, d>::Serialize(Ar);
		
		Ar << MP;

		Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);
		if (Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) >= FUE5ReleaseStreamObjectVersion::SinglePrecisonParticleDataPT)
		{
			Ar << MQ << MPreV << MPreW; 
		}
		else
		{
			TArrayCollectionArray<TRotation<FReal, d>> QDouble;
			QDouble.Resize(MQ.Num());
			for (int32 Index = 0; Index < MQ.Num(); ++Index)
			{
				QDouble[Index] = TRotation<FReal, d >(MQ[Index]);
			}
			TArrayCollectionArray<TVector<FReal, d>> PreVDouble;
			PreVDouble.Resize(MPreV.Num());
			for (int32 Index = 0; Index < MPreV.Num(); ++Index)
			{
				PreVDouble[Index] = TVector<FReal, d >(MPreV[Index]);
			}
			TArrayCollectionArray<TVector<FReal, d>> PreWDouble;
			PreWDouble.Resize(MPreW.Num());
			for (int32 Index = 0; Index < MPreW.Num(); ++Index)
			{
				PreWDouble[Index] = TVector<FReal, d >(MPreW[Index]);
			}

			Ar << QDouble << PreVDouble << PreWDouble;

			MQ.Resize(QDouble.Num());
			for (int32 Index = 0; Index < QDouble.Num(); ++Index)
			{
				MQ[Index] = TRotation<FRealSingle, d >(QDouble[Index]);
			}
			MPreV.Resize(PreVDouble.Num());
			for (int32 Index = 0; Index < PreVDouble.Num(); ++Index)
			{
				MPreV[Index] = TVector<FRealSingle, d >(PreVDouble[Index]);
			}
			MPreW.Resize(PreWDouble.Num());
			for (int32 Index = 0; Index < PreWDouble.Num(); ++Index)
			{
				MPreW[Index] = TVector<FRealSingle, d >(PreWDouble[Index]);
			}
		}
	}

  private:
	TArrayCollectionArray<TVector<T, d>> MP;
	TArrayCollectionArray<TRotation<FRealSingle, d>> MQ;
	TArrayCollectionArray<TVector<FRealSingle, d>> MPreV;
	TArrayCollectionArray<TVector<FRealSingle, d>> MPreW;
	TArrayCollectionArray<int32> MSolverBodyIndex;	// Transient for use in constraint solver
};

using FPBDRigidParticles = TPBDRigidParticles<FReal, 3>;

template <typename T, int d>
FChaosArchive& operator<<(FChaosArchive& Ar, TPBDRigidParticles<T, d>& Particles)
{
	Particles.Serialize(Ar);
	return Ar;
}
}

#if defined(_MSC_VER)
#pragma warning(pop)
#endif
