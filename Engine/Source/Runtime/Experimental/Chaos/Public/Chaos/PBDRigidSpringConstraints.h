// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Array.h"
#include "Chaos/Evolution/IndexedConstraintContainer.h"
#include "Chaos/ParticleHandle.h"

namespace Chaos
{
	class FPBDRigidSpringConstraints;

	class FPBDRigidSpringConstraintHandle final : public TIndexedContainerConstraintHandle<FPBDRigidSpringConstraints>
	{
	public:
		using Base = TIndexedContainerConstraintHandle<FPBDRigidSpringConstraints>;
		using FConstraintContainer = FPBDRigidSpringConstraints;
		using FGeometryParticleHandle = TGeometryParticleHandle<FReal, 3>;

		FPBDRigidSpringConstraintHandle()
		{
		}
		
		FPBDRigidSpringConstraintHandle(FConstraintContainer* InConstraintContainer, int32 InConstraintIndex) 
		: TIndexedContainerConstraintHandle<FPBDRigidSpringConstraints>(InConstraintContainer, InConstraintIndex)
		{
		}

		CHAOS_API const TVector<FVec3, 2>& GetConstraintPositions() const;
		CHAOS_API void SetConstraintPositions(const TVector<FVec3, 2>& ConstraintPositions);
		
		CHAOS_API virtual FParticlePair GetConstrainedParticles() const override final;

		// Get the rest length of the spring
		CHAOS_API FReal GetRestLength() const;
		CHAOS_API void SetRestLength(const FReal SpringLength);

		static const FConstraintHandleTypeID& StaticType()
		{
			static FConstraintHandleTypeID STypeID(TEXT("FRigidSpringConstraintHandle"), &FIndexedConstraintHandle::StaticType());
			return STypeID;
		}
	};


	class FPBDRigidSpringConstraints : public TPBDIndexedConstraintContainer<FPBDRigidSpringConstraints>
	{
	public:
		// @todo(ccaulfield): an alternative AddConstraint which takes the constrain settings rather than assuming everything is in world-space rest pose

		using Base = TPBDIndexedConstraintContainer<FPBDRigidSpringConstraints>;
		using FConstraintContainerHandle = FPBDRigidSpringConstraintHandle;
		using FConstraintHandleAllocator = TConstraintHandleAllocator<FPBDRigidSpringConstraints>;
		using FConstrainedParticlePair = TVector<TGeometryParticleHandle<FReal, 3>*, 2>;
		using FHandles = TArray<FConstraintContainerHandle*>;

		FPBDRigidSpringConstraints();
		virtual ~FPBDRigidSpringConstraints();
		//
		// Constraint Container API
		//

		/**
		 * Get the number of constraints.
		 */
		int32 NumConstraints() const
		{
			return Constraints.Num();
		}

		/**
		 * Add a constraint initialized from current world-space particle positions.
		 * You would use this method when your objects are already positioned in the world.
		 *
		 * \param InConstrainedParticles the two particles connected by the spring
		 * \param InLocations the world-space locations of the spring connectors on each particle
		 */
		FConstraintContainerHandle* AddConstraint(const FConstrainedParticlePair& InConstrainedParticles, const  TVector<FVec3, 2>& InLocations, FReal Stiffness, FReal Damping, FReal RestLength);

		/**
		 * Remove the specified constraint.
		 */
		void RemoveConstraint(int ConstraintIndex);

		/**
		 * Disabled the specified constraint.
		 */
		void DisableConstraints(const TSet<TGeometryParticleHandle<FReal, 3>*>& RemovedParticles) 
		{
			// @todo(chaos)
		}


		//
		// Constraint API
		//
		FHandles& GetConstraintHandles()
		{
			return Handles;
		}
		const FHandles& GetConstConstraintHandles() const
		{
			return Handles;
		}

		const FConstraintContainerHandle* GetConstraintHandle(int32 ConstraintIndex) const
		{
			return Handles[ConstraintIndex];
		}

		FConstraintContainerHandle* GetConstraintHandle(int32 ConstraintIndex)
		{
			return Handles[ConstraintIndex];
		}

		/**
		 * Get the particles that are affected by the specified constraint.
		 */
		const FConstrainedParticlePair& GetConstrainedParticles(int32 ConstraintIndex) const
		{
			return Constraints[ConstraintIndex];
		}

		/**
		 * Get the local-space constraint positions for each body.
		 */
		const TVector<FVec3, 2>& GetConstraintPositions(int ConstraintIndex) const
		{
			return Distances[ConstraintIndex];
		}

		/**
		 * Set the local-space constraint positions for each body.
		 */
		void SetConstraintPositions(int ConstraintIndex, const TVector<FVec3, 2>& ConstraintPositions)
		{
			Distances[ConstraintIndex] = ConstraintPositions;
		}

		/**
		 * Get the rest length of the spring
		 */
		FReal GetRestLength(int32 ConstraintIndex) const
		{
			return SpringSettings[ConstraintIndex].RestLength;
		}

		/**
		 * Set the rest length of the spring
		 */
		void SetRestLength(int32 ConstraintIndex, const FReal SpringLength)
		{
			SpringSettings[ConstraintIndex].RestLength = SpringLength;
		}


		//
		// FConstraintContainer Implementation
		//
		virtual int32 GetNumConstraints() const override final { return NumConstraints(); }
		virtual void ResetConstraints() override final {}
		virtual void AddConstraintsToGraph(Private::FPBDIslandManager& IslandManager) override final;
		virtual void PrepareTick() override final {}
		virtual void UnprepareTick() override final {}

		//
		// TSimpleConstraintContainerSolver API - used by RBAN solvers
		//
		void AddBodies(FSolverBodyContainer& SolverBodyContainer);
		void GatherInput(const FReal Dt) {}
		void ScatterOutput(const FReal Dt);
		void ApplyPositionConstraints(const FReal Dt, const int32 It, const int32 NumIts);
		void ApplyVelocityConstraints(const FReal Dt, const int32 It, const int32 NumIts) {}
		void ApplyProjectionConstraints(const FReal Dt, const int32 It, const int32 NumIts) {}

		//
		// TIndexedConstraintContainerSolver API - used by World solvers
		//
		void AddBodies(const TArrayView<int32>& ConstraintIndices, FSolverBodyContainer& SolverBodyContainer);
		void GatherInput(const TArrayView<int32>& ConstraintIndices, const FReal Dt) {}
		void ScatterOutput(const TArrayView<int32>& ConstraintIndices, const FReal Dt);
		void ApplyPositionConstraints(const TArrayView<int32>& ConstraintIndices, const FReal Dt, const int32 It, const int32 NumIts);
		void ApplyVelocityConstraints(const TArrayView<int32>& ConstraintIndices, const FReal Dt, const int32 It, const int32 NumIts) {}
		void ApplyProjectionConstraints(const TArrayView<int32>& ConstraintIndices, const FReal Dt, const int32 It, const int32 NumIts) {}

	protected:
		using Base::GetConstraintIndex;
		using Base::SetConstraintIndex;

	private:
		void AddBodies(const int32 ConstraintIndex, FSolverBodyContainer& SolverBodyContainer);
		void ApplyPhase1Single(const FReal Dt, int32 ConstraintIndex) const;

		void InitDistance(int32 ConstraintIndex, const FVec3& Location0, const FVec3& Location1);

		FVec3 GetDelta(int32 ConstraintIndex, const FVec3& WorldSpaceX1, const FVec3& WorldSpaceX2) const;

		struct FSpringSettings
		{
			FReal Stiffness;
			FReal Damping;
			FReal RestLength;
		};

		TArray<FConstrainedParticlePair> Constraints;
		TArray<TVector<FVec3, 2>> Distances;
		TArray<FSpringSettings> SpringSettings;

		TArray<FSolverBodyPtrPair> ConstraintSolverBodies;

		TArray<FConstraintContainerHandle*> Handles;
		FConstraintHandleAllocator HandleAllocator;
	};
}
