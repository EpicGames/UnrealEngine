// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Character/CharacterGroundConstraintContainer.h"
#include "Chaos/ChaosDebugName.h"
#include "Chaos/PBDCollisionConstraints.h"
#include "Chaos/PBDRigidClustering.h"
#include "Chaos/PBDRigidParticles.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/Transform.h"
#include "Chaos/Framework/DebugSubstep.h"
#include "HAL/Event.h"
#include "Chaos/PBDJointConstraints.h"
#include "Chaos/PBDRigidsEvolutionFwd.h"
#include "Chaos/PBDRigidsSOAs.h"
#include "Chaos/SpatialAccelerationCollection.h"
#include "Chaos/TaskDispatcherEvolution.h"
#include "Chaos/Island/IslandManager.h"
#include "Chaos/Island/IslandGroupManager.h"
#include "Chaos/Defines.h"
#include "Chaos/PendingSpatialData.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "RewindData.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "ChaosVisualDebugger/ChaosVDContextProvider.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8


// Enable support for Collision Test Mode (reset particle positions and constraints every tick for debugging)
#define CHAOS_EVOLUTION_COLLISION_TESTMODE (!UE_BUILD_TEST && !UE_BUILD_SHIPPING)

extern int32 ChaosRigidsEvolutionApplyAllowEarlyOutCVar;
extern int32 ChaosRigidsEvolutionApplyPushoutAllowEarlyOutCVar;
extern int32 ChaosNumPushOutIterationsOverride;
extern int32 ChaosNumContactIterationsOverride;

namespace Chaos
{
extern CHAOS_API int32 ForceNoCollisionIntoSQ;

struct FBroadPhaseConfig
{
	enum 
	{
		Grid = 0,
		Tree = 1,
		TreeOfGrid = 2,
		TreeAndGrid = 3,
		TreeOfGridAndGrid = 4
	};

	// Broadphase Type from the enum above
	int32 BroadphaseType;
	int32 BVNumCells;
	int32 MaxChildrenInLeaf;
	int32 MaxTreeDepth;
	int32 AABBMaxChildrenInLeaf;
	int32 AABBMaxTreeDepth;
	FRealSingle MaxPayloadSize;
	int32 IterationsPerTimeSlice;

	FBroadPhaseConfig()
	{
		BroadphaseType = Tree;
		BVNumCells = 35;
		MaxChildrenInLeaf = 5;
		MaxTreeDepth = 200;
		AABBMaxChildrenInLeaf = 500;
		AABBMaxTreeDepth = 200;
		MaxPayloadSize = 100000;
		IterationsPerTimeSlice = 4000;
	}
};

extern CHAOS_API FBroadPhaseConfig BroadPhaseConfig;

extern CHAOS_API int32 FixBadAccelerationStructureRemoval;

class FChaosArchive;

template <typename TPayload, typename T, int d>
class ISpatialAccelerationCollection;

struct FEvolutionStats
{
	int32 ActiveCollisionPoints;
	int32 ActiveShapes;
	int32 ShapesForAllConstraints;
	int32 CollisionPointsForAllConstraints;

	FEvolutionStats()
	{
		Reset();
	}

	void Reset()
	{
		ActiveCollisionPoints = 0;
		ActiveShapes = 0;
		ShapesForAllConstraints = 0;
		CollisionPointsForAllConstraints = 0;
	}

	FEvolutionStats& operator+=(const FEvolutionStats& Other)
	{
		ActiveCollisionPoints += Other.ActiveCollisionPoints;
		ActiveShapes += Other.ActiveShapes;
		ShapesForAllConstraints += Other.ShapesForAllConstraints;
		CollisionPointsForAllConstraints += Other.CollisionPointsForAllConstraints;
		return *this;
	}
};

struct FSpatialAccelerationCacheHandle;

/** The SOA cache used for a single acceleration structure */
class FSpatialAccelerationCache : public TArrayCollection
{
public:
	using THandleType = FSpatialAccelerationCacheHandle;

	FSpatialAccelerationCache()
	{
		AddArray(&MHasBoundingBoxes);
		AddArray(&MBounds);
		AddArray(&MPayloads);

#if PARTICLE_ITERATOR_RANGED_FOR_CHECK
		MDirtyValidationCount = 0;
#endif
	}

	FSpatialAccelerationCache(const FSpatialAccelerationCache&) = delete;
	FSpatialAccelerationCache(FSpatialAccelerationCache&& Other)
		: TArrayCollection()
		, MHasBoundingBoxes(MoveTemp(Other.MHasBoundingBoxes))
		, MBounds(MoveTemp(Other.MBounds))
		, MPayloads(MoveTemp(Other.MPayloads))
	{
		ResizeHelper(Other.MSize);
		Other.MSize = 0;

		AddArray(&MHasBoundingBoxes);
		AddArray(&MBounds);
		AddArray(&MPayloads);
#if PARTICLE_ITERATOR_RANGED_FOR_CHECK
		MDirtyValidationCount = 0;
#endif
	}

	FSpatialAccelerationCache& operator=(FSpatialAccelerationCache&& Other)
	{
		if (&Other != this)
		{
			MHasBoundingBoxes = MoveTemp(Other.MHasBoundingBoxes);
			MBounds = MoveTemp(Other.MBounds);
			MPayloads = MoveTemp(Other.MPayloads);
			ResizeHelper(Other.MSize);
			Other.MSize = 0;
#if PARTICLE_ITERATOR_RANGED_FOR_CHECK
			MDirtyValidationCount = 0;
			++Other.MDirtyValidationCount;
#endif
		}

		return *this;
	}

#if PARTICLE_ITERATOR_RANGED_FOR_CHECK
	int32 DirtyValidationCount() const { return MDirtyValidationCount; }
#endif

	void AddElements(const int32 Num)
	{
		AddElementsHelper(Num);
		IncrementDirtyValidation();
	}

	void DestroyElement(const int32 Idx)
	{
		RemoveAtSwapHelper(Idx);
		IncrementDirtyValidation();
	}

	bool HasBounds(const int32 Idx) const { return MHasBoundingBoxes[Idx]; }
	bool& HasBounds(const int32 Idx) { return MHasBoundingBoxes[Idx]; }

	const FAABB3& Bounds(const int32 Idx) const { return MBounds[Idx]; }
	FAABB3& Bounds(const int32 Idx) { return MBounds[Idx]; }

	const FAccelerationStructureHandle& Payload(const int32 Idx) const { return MPayloads[Idx]; }
	FAccelerationStructureHandle& Payload(const int32 Idx) { return MPayloads[Idx]; }

private:
	void IncrementDirtyValidation()
	{
#if PARTICLE_ITERATOR_RANGED_FOR_CHECK
		++MDirtyValidationCount;
#endif
	}

	TArrayCollectionArray<bool> MHasBoundingBoxes;
	TArrayCollectionArray<FAABB3> MBounds;
	TArrayCollectionArray<FAccelerationStructureHandle> MPayloads;

#if PARTICLE_ITERATOR_RANGED_FOR_CHECK
	int32 MDirtyValidationCount;
#endif
};

/** The handle the acceleration structure uses to access the data (similar to particle handle) */
struct FSpatialAccelerationCacheHandle
{
	using THandleBase = FSpatialAccelerationCacheHandle;
	using TTransientHandle = FSpatialAccelerationCacheHandle;

	FSpatialAccelerationCacheHandle(FSpatialAccelerationCache* InCache = nullptr, int32 InEntryIdx = INDEX_NONE)
		: Cache(InCache)
		, EntryIdx(InEntryIdx)
	{}

	template <typename TPayloadType>
	TPayloadType GetPayload(int32 Idx) const
	{
		return Cache->Payload(EntryIdx);
	}

	bool HasBoundingBox() const
	{
		return Cache->HasBounds(EntryIdx);
	}

	const FAABB3& BoundingBox() const
	{
		return Cache->Bounds(EntryIdx);
	}

	bool LightWeightDisabled() const { return false; }

	union
	{
		FSpatialAccelerationCache* GeometryParticles;	//using same name as particles SOA for template reuse, should probably rethink this
		FSpatialAccelerationCache* Cache;
	};

	union
	{
		int32 ParticleIdx;	//same name for template reasons. Not really a particle idx
		int32 EntryIdx;
	};
};

struct ISpatialAccelerationCollectionFactory
{
	//Create an empty acceleration collection with the desired buckets. Chaos enqueues acceleration structure operations per bucket
	virtual TUniquePtr<ISpatialAccelerationCollection<FAccelerationStructureHandle, FReal, 3>> CreateEmptyCollection() = 0;

	// Determines if bucket implements time slicing.
	virtual bool IsBucketTimeSliced(uint16 BucketIdx) const = 0;

	//Chaos creates new acceleration structures per bucket. Factory can change underlying type at runtime as well as number of buckets to AB test
	virtual TUniquePtr<ISpatialAcceleration<FAccelerationStructureHandle, FReal, 3>> CreateAccelerationPerBucket_Threaded(const TConstParticleView<FSpatialAccelerationCache>& Particles, uint16 BucketIdx, bool ForceFullBuild, bool bDynamicTree, bool bBuildOverlapCache) = 0;

	//Mask indicating which bucket is active. Spatial indices in inactive buckets fallback to bucket 0. Bit 0 indicates bucket 0 is active, Bit 1 indicates bucket 1 is active, etc...
	virtual uint8 GetActiveBucketsMask() const = 0;

	//Serialize the collection in and out
	virtual void Serialize(TUniquePtr<ISpatialAccelerationCollection<FAccelerationStructureHandle, FReal, 3>>& Ptr, FChaosArchive& Ar) = 0;

	virtual ~ISpatialAccelerationCollectionFactory() = default;
};

class FPBDRigidsEvolutionBase
{
public:
	using FAccelerationStructure = ISpatialAccelerationCollection<FAccelerationStructureHandle,FReal,3>;

	typedef TFunction<void(TTransientPBDRigidParticleHandle<FReal, 3>& Particle, const FReal)> FForceRule;
	typedef TFunction<void(const TArray<FGeometryParticleHandle*>&, const FReal)> FUpdateVelocityRule;
	typedef TFunction<void(const TParticleView<FPBDRigidParticles>&, const FReal)> FUpdatePositionRule;
	typedef TFunction<void(FPBDRigidParticles&, const FReal, const FReal, const int32)> FKinematicUpdateRule;
	typedef TFunction<void(TParticleView<FPBDRigidParticles>&)> FCaptureRewindRule;
	typedef TFunction<void(TParticleView<FPBDRigidParticles>&)> FCaptureRewindKinematicTargetRule;

	CHAOS_API FPBDRigidsEvolutionBase(FPBDRigidsSOAs& InParticles, THandleArray<FChaosPhysicsMaterial>& InSolverPhysicsMaterials, bool InIsSingleThreaded = false);
	CHAOS_API virtual ~FPBDRigidsEvolutionBase();

	TArray<FGeometryParticleHandle*> CreateStaticParticles(int32 NumParticles, const FUniqueIdx* ExistingIndices = nullptr, const FGeometryParticleParameters& Params = FGeometryParticleParameters())
	{
		auto NewParticles = Particles.CreateStaticParticles(NumParticles, ExistingIndices, Params);
		for (auto& Particle : NewParticles)
		{
			DirtyParticle(*Particle);
		}
		return NewParticles;
	}

	TArray<FKinematicGeometryParticleHandle*> CreateKinematicParticles(int32 NumParticles, const FUniqueIdx* ExistingIndices = nullptr, const FKinematicGeometryParticleParameters& Params = FKinematicGeometryParticleParameters())
	{
		auto NewParticles = Particles.CreateKinematicParticles(NumParticles, ExistingIndices, Params);
		for (auto& Particle : NewParticles)
		{
			DirtyParticle(*Particle);
		}
		return NewParticles;
	}

	TArray<FPBDRigidParticleHandle*> CreateDynamicParticles(int32 NumParticles, const FUniqueIdx* ExistingIndices = nullptr, const FPBDRigidParticleParameters& Params = FPBDRigidParticleParameters())
	{
		auto NewParticles = Particles.CreateDynamicParticles(NumParticles, ExistingIndices, Params);
		for (auto& Particle : NewParticles)
		{
			DirtyParticle(*Particle);
		}
		return NewParticles;
	}

	TArray<TPBDRigidClusteredParticleHandle<FReal, 3>*> CreateClusteredParticles(int32 NumParticles,const FUniqueIdx* ExistingIndices = nullptr,  const FPBDRigidParticleParameters& Params = FPBDRigidParticleParameters())
	{
		auto NewParticles = Particles.CreateClusteredParticles(NumParticles, ExistingIndices, Params);
		for (auto& Particle : NewParticles)
		{
			DirtyParticle(*Particle);
		}
		return NewParticles;
	}

	TArray<TPBDGeometryCollectionParticleHandle<FReal, 3>*> CreateGeometryCollectionParticles(int32 NumParticles,const FUniqueIdx* ExistingIndices = nullptr,  const FPBDRigidParticleParameters& Params = FPBDRigidParticleParameters())
	{
		auto NewParticles = Particles.CreateGeometryCollectionParticles(NumParticles, ExistingIndices, Params);
		for (auto& Particle : NewParticles)
		{
			DirtyParticle(*Particle);
		}
		return NewParticles;
	}

	void AddForceFunction(FForceRule ForceFunction) { ForceRules.Add(ForceFunction); }
	void AddImpulseFunction(FForceRule ImpulseFunction) { ImpulseRules.Add(ImpulseFunction); }
	void SetParticleUpdatePositionFunction(FUpdatePositionRule ParticleUpdate) { ParticleUpdatePosition = ParticleUpdate; }
	void SetCaptureRewindDataFunction(FCaptureRewindRule Rule) { CaptureRewindData = Rule; }
	void SetCaptureRewindKinematicTargetFunction(FCaptureRewindKinematicTargetRule Rule) { CaptureRewindKinematicTarget = Rule; }

	TGeometryParticleHandles<FReal, 3>& GetParticleHandles() { return Particles.GetParticleHandles(); }
	const TGeometryParticleHandles<FReal, 3>& GetParticleHandles() const { return Particles.GetParticleHandles(); }

	FPBDRigidsSOAs& GetParticles() { return Particles; }
	const FPBDRigidsSOAs& GetParticles() const { return Particles; }

	/** Reset the collisions warm starting when resimulate. Ideally we should store
		  that in the RewindData history but probably too expensive for now */
	virtual void ResetCollisions() {};

	/**
	* Register a constraint container with the evolution. Constraints added to the container will be solved during the tick.
	* @note we do not currently support removing containers. In a few places we assume the ContainerId is persistent and equal to the array index.
	*/
	void AddConstraintContainer(FPBDConstraintContainer& InContainer, const int32 Priority = 0)
	{
		const int32 ContainerId = ConstraintContainers.Add(&InContainer);
		InContainer.SetContainerId(ContainerId);

		GetIslandManager().AddConstraintContainer(InContainer);
		IslandGroupManager.AddConstraintContainer(InContainer, Priority);
	}

	/**
	* Set the number of iterations to perform in the constraint position-solve phase
	*/
	void SetNumPositionIterations(int32 InNumIterations)
	{
		IslandGroupManager.SetNumPositionIterations(InNumIterations);
	}

	/**
	* Get the number of position iterations the solver is running
	*/
	int32 GetNumPositionIterations() const
	{
		return IslandGroupManager.GetIterationSettings().GetNumPositionIterations();
	}

	/**
	* Set the number of iterations to perform in the constraint velocity-solve phase
	*/
	void SetNumVelocityIterations(int32 InNumIterations)
	{
		IslandGroupManager.SetNumVelocityIterations(InNumIterations);
	}

	/**
	* Get the number of velocity iterations the solver is running
	*/
	int32 GetNumVelocityIterations() const
	{
		return IslandGroupManager.GetIterationSettings().GetNumVelocityIterations();
	}

	/**
	* Set the number of iterations to perform in the constraint projection phase
	*/
	void SetNumProjectionIterations(int32 InNumIterations)
	{
		IslandGroupManager.SetNumProjectionIterations(InNumIterations);
	}

	/**
	* Get the number of projection iterations the solver is running
	*/
	int32 GetNumProjectionIterations() const
	{
		return IslandGroupManager.GetIterationSettings().GetNumProjectionIterations();
	}

	/**
	* To be called after creating a particle in the Particles container
	* @todo(chaos): We should add a particle creation API to the evolution
	* @todo(chaos): This is (or could be) very similar to Enable/Disable - do we really need both?
	*/
	void RegisterParticle(FGeometryParticleHandle* Particle)
	{
		// Add to the graph if necessary. Only enabled dynamic particles are added at this stage. Kinematics
		// and statics are ignored until referenced by a constraint.
		if (FPBDRigidParticleHandle* Rigid = Particle->CastToRigidParticle())
		{
			if (Rigid->IsDynamic() && !Rigid->Disabled())
			{
				IslandManager.AddParticle(Particle);
			}
		}

		CVD_TRACE_PARTICLE(Particle);

		// Flag as dirty to update the spatial query acceleration structures
		DirtyParticle(*Particle);
	}

	/**
	* Enable a particle.Only enabled particles are simulated.
	* If the particle has constraints connected to it they will also be enabled (assuming the other particles in the constraints are also enabled). 
	*/
	void EnableParticle(FGeometryParticleHandle* Particle)
	{
		Particles.EnableParticle(Particle);
		EnableConstraints(Particle);
		IslandManager.AddParticle(Particle);
		DirtyParticle(*Particle, EPendingSpatialDataOperation::Add);
	}

	/**
	* Disable a particle so that it is no longer simulated. This also disables all constraints connected to the particle.
	*/
	void DisableParticle(FGeometryParticleHandle* Particle)
	{
#if CHAOS_EVOLUTION_COLLISION_TESTMODE
		TestModeParticleDisabled(Particle);
#endif

		// NOTE: kinematics must visit their graph edges to determine what islands they are in, so we must remove the 
		// particle from the graph before we disable its constraints or we don't know what island(s) to wake.
		IslandManager.RemoveParticle(Particle);

		RemoveParticleFromAccelerationStructure(*Particle);
		Particles.DisableParticle(Particle);
		DisableConstraints(Particle);
		DestroyTransientConstraints(Particle);

		if (FPBDRigidParticleHandle* Rigid = Particle->CastToRigidParticle())
		{
			// This flag is only updated for moving kinematics, so make sure 
			// we don't leave a residual value if we get enabled again
			Rigid->ClearIsMovingKinematic();
		}

		CVD_TRACE_PARTICLE(Particle);
	}

	/**
	* To be called when a particle geometry changes. We must clear collisions and anything else that may reference the prior shapes.
	*/
	void InvalidateParticle(FGeometryParticleHandle* Particle)
	{
		// Remove all constraints (collisions, joints etc) from the graph
		IslandManager.RemoveParticleConstraints(Particle);

		// Destroy all the transient constraints (collisions) because the particle has changed somehow (e.g. new shapes) and they may have cached the previous state
		// @todo(chaos): if any other types depended on geometry, they would also need to be notified here. Maybe make this more specific and tell all types...
		DestroyTransientConstraints(Particle);
	}
	
	CHAOS_API void FlushExternalAccelerationQueue(FAccelerationStructure& Acceleration,FPendingSpatialDataQueue& ExternalQueue);

	void DisableParticles(TSet<FGeometryParticleHandle*> &ParticlesIn)
	{
		for (FGeometryParticleHandle* Particle : ParticlesIn)
		{
			DisableParticle(Particle);
		}
	}

	template <bool bPersistent>
	FORCEINLINE_DEBUGGABLE bool IsADisableCluster(TGeometryParticleHandleImp<FReal, 3, bPersistent>& Particle)
	{
		const TPBDRigidParticleHandleImp<FReal, 3, bPersistent>* AsRigid = Particle.CastToRigidParticle();
		if (AsRigid && AsRigid->Disabled())
		{
			TPBDRigidClusteredParticleHandleImp<FReal, 3, bPersistent>* AsClustered = Particle.CastToClustered();

			if (AsClustered)
			{
				// For clustered particles, they may appear disabled but they're being driven by an internal (solver-owned) cluster parent.
				// If this is the case we let the spatial data update with those particles, otherwise skip.
				// Alternatively, the particle may be being driven by a cluster union. Disabled children of a cluster union should not be added
				// to the SQ.
				// #BGTODO consider converting MDisabled into a bitfield for multiple disable types (Disabled, DisabledDriven, etc.)
				if (FPBDRigidParticleHandle* ClusterParentBase = AsClustered->ClusterIds().Id)
				{
					if (Chaos::TPBDRigidClusteredParticleHandle<FReal, 3>*ClusterParent = ClusterParentBase->CastToClustered())
					{
						if (!ClusterParent->InternalCluster())
						{
							return true;
						}

						// We know we're an internal cluster now. If this was a GC internal proxy, we'd expect the 
						// parent's proxy to be the same as the input particle's. If this is not the case, we know
						// we're in a cluster union.
						if (ClusterParent->PhysicsProxy() != Particle.PhysicsProxy())
						{
							return true;
						}
					}
					else
					{
						// There's probably no way we get here since if we're clustering the parent should always be a clustered.
						ensure(false);
						return true;
					}
				}
				else
				{
					// Disabled cluster particle that doesn't have a parent cluster. MUST BE IGNORED.
					return true;
				}
			}
			else
			{
				// Disabled particles take no immediate part in sim or query so shouldn't be added to the acceleration
				return true;
			}
		}
		return false;
	}

	template <bool bPersistent>
	FORCEINLINE_DEBUGGABLE void DirtyParticle(TGeometryParticleHandleImp<FReal, 3, bPersistent>& Particle, const EPendingSpatialDataOperation Op = EPendingSpatialDataOperation::Update)
	{
		ensure(Op != EPendingSpatialDataOperation::Delete); // Don't use the function to delete particles
		
		if (IsADisableCluster(Particle))
		{
			return;
		}

		//only add to acceleration structure if it has collision
		if (Particle.HasCollision() || ForceNoCollisionIntoSQ)
		{
			//TODO: distinguish between new particles and dirty particles - Adds and updates are treated the same right now
			const FUniqueIdx UniqueIdx = Particle.UniqueIdx();
			FPendingSpatialData& PendingData = InternalAccelerationQueue.Add(0, UniqueIdx);
			PendingData.Operation = Op;
			PendingData.AccelerationHandle = FAccelerationStructureHandle(Particle);
			PendingData.SpatialIdx = Particle.SpatialIdx();


			FPendingSpatialData& AsyncSpatialData = AsyncAccelerationQueue.FindOrAdd(UniqueIdx);
			AsyncSpatialData = PendingData;
			// ensure(AsyncSpatialData.Operation != EPendingSpatialDataOperation::Delete); // TODO: This may be hit: Potentially due to UniqueIdx reuse?
		}
	}

	template <bool bPersistent>
	FORCEINLINE_DEBUGGABLE void DirtyParticleAsync(TGeometryParticleHandleImp<FReal, 3, bPersistent>& Particle, const EPendingSpatialDataOperation Op = EPendingSpatialDataOperation::Update)
	{
		ensure(Op != EPendingSpatialDataOperation::Delete); // Don't use the function to delete particles

		if (IsADisableCluster(Particle))
		{
			return;
		}

		//only add to acceleration structure if it has collision
		if (Particle.HasCollision() || ForceNoCollisionIntoSQ)
		{
			//TODO: distinguish between new particles and dirty particles - Adds and updates are treated the same right now
			const FUniqueIdx UniqueIdx = Particle.UniqueIdx();


			FPendingSpatialData& AsyncSpatialData = AsyncAccelerationQueue.FindOrAdd(UniqueIdx);
			AsyncSpatialData.Operation = Op;
			AsyncSpatialData.AccelerationHandle = FAccelerationStructureHandle(Particle);
			AsyncSpatialData.SpatialIdx = Particle.SpatialIdx();

			// ensure(AsyncSpatialData.Operation != EPendingSpatialDataOperation::Delete); // TODO: This may be hit: Potentially due to UniqueIdx reuse?
		}
	}

	void DestroyParticle(FGeometryParticleHandle* Particle)
	{
#if CHAOS_EVOLUTION_COLLISION_TESTMODE
		TestModeParticleDisabled(Particle);
#endif

		if (MRewindData)
		{
			MRewindData->RemoveObject(Particle);
		}

		RemoveParticleFromAccelerationStructure(*Particle);
		DisconnectPersistentConstraints(TSet<FGeometryParticleHandle*>({ Particle }));
		DestroyTransientConstraints(Particle);
		IslandManager.RemoveParticle(Particle);
		Particles.DestroyParticle(Particle);
	}

	/**
	 * Preallocate buffers for creating \p Num particles.
	 */
	void ReserveParticles(const int32 Num)
	{
		if (const int32 NumNew = IslandManager.ReserveParticles(Num))
		{
			AsyncAccelerationQueue.PendingData.Reserve(AsyncAccelerationQueue.Num() + NumNew);
		}
	}

	CHAOS_API void SetParticleObjectState(FPBDRigidParticleHandle* Particle, EObjectStateType ObjectState);
	
	/** Change the object state for a list of particles */
	CHAOS_API void SetParticlesObjectState(TArrayView<FPBDRigidParticleHandle*> Particles, const EObjectStateType ObjectState, const bool bUpdateViews);

	// Wake a dynamic particle and reset sleep counters for its island
	CHAOS_API void WakeParticle(FPBDRigidParticleHandle* Particle);

	CHAOS_API void SetParticleSleepType(FPBDRigidParticleHandle* Particle, ESleepType InSleepType);

	CHAOS_API void DisableParticles(const TSet<FGeometryParticleHandle*>& InParticles);

	/** remove a constraint from the constraint graph */
	void RemoveConstraintFromConstraintGraph(FConstraintHandle* ConstraintHandle)
	{
		if (ConstraintHandle->IsInConstraintGraph())
		{
			IslandManager.RemoveConstraint(ConstraintHandle);
		}
	}

	/** remove a list of constraints from the constraint graph */
	void RemoveConstraintsFromConstraintGraph(const FConstraintHandleArray& Constraints)
	{
		for (FConstraintHandle* ConstraintHandle : Constraints)
		{
			RemoveConstraintFromConstraintGraph(ConstraintHandle);
		}
	}

	/** 
	* Disconnect constraints (all types except collisions) from a set of particles to be destroyed. 
	* this will set the constraints to Enabled = false and set their respective bodies handles to nullptr.
	* Once this is done, the constraints cannot be re-enabled.
	* @note This only applies to persistent constraints (joints etc), not transient constraints (collisions)
	* @see DestroyTransientConstraints()
	*/
	UE_DEPRECATED(5.8, "Use DisconnectPersistentConstraints") void DisconnectConstraints(const TSet<FGeometryParticleHandle*>& RemovedParticles) { DisconnectPersistentConstraints(RemovedParticles); }
	void DisconnectPersistentConstraints(const TSet<FGeometryParticleHandle*>& RemovedParticles)
	{
		for (FPBDConstraintContainer* Container : ConstraintContainers)
		{
			Container->DisconnectConstraints(RemovedParticles);
		}

		for (FGeometryParticleHandle* ParticleHandle : RemovedParticles)
		{
			RemoveConstraintsFromConstraintGraph(ParticleHandle->ParticleConstraints());
			ParticleHandle->ParticleConstraints().Reset();
		}
	}

	/** 
	* Disable constraints (all types except collisions) from a particle to be removed (or destroyed)
	* this will set the constraints to Enabled = false, but leave connections to the particles to support
	* re-enabling at a later time.
	* @note This only applies to persistent constraints (joints etc), not transient constraints (collisions)
	* @see DestroyTransientConstraints()
	*/
	void DisableConstraints(FGeometryParticleHandle* ParticleHandle)
	{
		for (FPBDConstraintContainer* Container : ConstraintContainers)
		{
			Container->OnDisableParticle(ParticleHandle);
		}

		RemoveConstraintsFromConstraintGraph(ParticleHandle->ParticleConstraints());
	}

	/** 
	* Enable constraints (all types except collisions) from the enabled particles; constraints will only become enabled if their particle end points are valid.
	* @note This only applies to persistent constraints (joints etc), not transient constraints (collisions)
	*/
	void EnableConstraints(FGeometryParticleHandle* ParticleHandle)
	{
		for (FPBDConstraintContainer* Container : ConstraintContainers)
		{
			Container->OnEnableParticle(ParticleHandle);
		}
	}

	/** 
	* Clear all constraints from the system reeady for shut down 
	*/
	void ResetConstraints()
	{
		// Remove all the constraints from the graph
		GetIslandManager().Reset();

		// Clear all particle lists of collisions and constraints
		// (this could be performed by the constraint containers
		// but it would be unnecessarily expensive to remove them
		// one by one)
		for (auto& Particle : Particles.GetAllParticlesView())
		{
			Particle.ParticleConstraints().Reset();
			Particle.ParticleCollisions().Reset();
		}

		// Remove all constraints from the containers
		for (FPBDConstraintContainer* Container : ConstraintContainers)
		{
			Container->ResetConstraints();
		}
	}

	/**
	* Destroy all transient constraints (collisions) involving the specified particle.
	*/
	virtual void DestroyTransientConstraints(FGeometryParticleHandle* Particle) {}
	virtual void DestroyTransientConstraints() {}

	const TParticleView<FPBDRigidClusteredParticles>& GetNonDisabledClusteredView() const { return Particles.GetNonDisabledClusteredView(); }

	TSerializablePtr<FChaosPhysicsMaterial> GetPhysicsMaterial(const FGeometryParticleHandle* Particle) const { return Particle->AuxilaryValue(PhysicsMaterials); }

	CHAOS_API const FChaosPhysicsMaterial* GetFirstPhysicsMaterial(const FGeometryParticleHandle* Particle) const;
	
	const TUniquePtr<FChaosPhysicsMaterial> &GetPerParticlePhysicsMaterial(const FGeometryParticleHandle* Particle) const { return Particle->AuxilaryValue(PerParticlePhysicsMaterials); }

	void SetPerParticlePhysicsMaterial(FGeometryParticleHandle* Particle, TUniquePtr<FChaosPhysicsMaterial> &InMaterial)
	{
		Particle->AuxilaryValue(PerParticlePhysicsMaterials) = MoveTemp(InMaterial);
		IslandManager.UpdateParticleMaterial(Particle);
	}

	void SetPhysicsMaterial(FGeometryParticleHandle* Particle, TSerializablePtr<FChaosPhysicsMaterial> InMaterial)
	{
		check(!Particle->AuxilaryValue(PerParticlePhysicsMaterials)); //shouldn't be setting non unique material if a unique one already exists
		Particle->AuxilaryValue(PhysicsMaterials) = InMaterial;
		IslandManager.UpdateParticleMaterial(Particle);
	}

	void PrepareTick()
	{
		for (FPBDConstraintContainer* Container : ConstraintContainers)
		{
			Container->PrepareTick();
		}
	}

	void UnprepareTick()
	{
		for (FPBDConstraintContainer* Container : ConstraintContainers)
		{
			Container->UnprepareTick();
		}
	}

	virtual void ApplyKinematicTargets(const FReal Dt, const FReal StepFraction) {}

	/** Make a copy of the acceleration structure to allow for external modification.
	    This is needed for supporting sync operations on SQ structure from game thread. You probably want to go through solver which maintains PendingExternal */
	CHAOS_API void UpdateExternalAccelerationStructure_External(ISpatialAccelerationCollection<FAccelerationStructureHandle, FReal, 3>*& ExternalStructure, FPendingSpatialDataQueue& PendingExternal);

	ISpatialAccelerationCollection<FAccelerationStructureHandle, FReal, 3>* GetSpatialAcceleration() { return InternalAcceleration; }

	/** Perform a blocking flush of the spatial acceleration structure for situations where we aren't simulating but must have an up to date structure */
	CHAOS_API void FlushSpatialAcceleration();

	/** Rebuilds the spatial acceleration from scratch. This should only be used for perf testing */
	CHAOS_API void RebuildSpatialAccelerationForPerfTest();

	/* Ticks computation of acceleration structures. Normally handled by Advance, but if not advancing can be called to incrementally build structures.*/
	CHAOS_API void ComputeIntermediateSpatialAcceleration(bool bBlock = false);

	UE_DEPRECATED(5.2, "Renamed to GetIslandManager")
	const Private::FPBDIslandManager& GetConstraintGraph() const { return IslandManager; }
	UE_DEPRECATED(5.2, "Renamed to GetIslandManager")
	Private::FPBDIslandManager& GetConstraintGraph() { return IslandManager; }
	
	Private::FPBDIslandManager& GetIslandManager() { return IslandManager; }
	const Private::FPBDIslandManager& GetIslandManager() const { return IslandManager; }
	const Private::FPBDIslandGroupManager& GetIslandGroupManager() const { return IslandGroupManager; }


	void SetResim(bool bInResim) { bIsResim = bInResim; }
	const bool IsResimming() const { return bIsResim; }

	void SetReset(bool bInReset) { bIsReset = bInReset; }
	const bool IsResetting() const { return bIsReset; }

	void Serialize(FChaosArchive& Ar);

	FUniqueIdx GenerateUniqueIdx()
	{
		// NOTE: this should be thread safe since evolution has already been initialized on GT
		FUniqueIdx Result;
		
		// This uses thread safe atomics, so we have to do it in the open.
		UE_AUTORTFM_OPEN { Result = Particles.GetUniqueIndices().GenerateUniqueIdx(); };

		// But if we abort, we need to release the now unused index.
		AutoRTFM::OnAbort([this, Result] { this->ReleaseUniqueIdx(Result); });

		return Result;
	}

	void ReleaseUniqueIdx(FUniqueIdx UniqueIdx)
	{
		UniqueIndicesPendingRelease.Add(UniqueIdx);
	}

	bool IsUniqueIndexPendingRelease(FUniqueIdx UniqueIdx) const
	{
		return UniqueIndicesPendingRelease.Contains(UniqueIdx) || PendingReleaseIndices.Contains(UniqueIdx);
	}

	void KillSafeAsyncTasks()
	{
		if (AccelerationStructureTaskComplete.GetReference() && !AccelerationStructureTaskComplete->IsComplete() && bAccelerationStructureTaskSignalKill != nullptr)
		{
			*bAccelerationStructureTaskSignalKill = true;			
		}
	}

	bool AreAnyTasksPending() const
	{
		return AccelerationStructureTaskComplete.GetReference() && !AccelerationStructureTaskComplete->IsComplete() && 
			(bAccelerationStructureTaskSignalKill == nullptr || bAccelerationStructureTaskStarted  == nullptr || *bAccelerationStructureTaskSignalKill == false || *bAccelerationStructureTaskStarted == true);
	}

	void SetCanStartAsyncTasks(bool bInCanStartAsyncTasks)
	{
		bCanStartAsyncTasks = bInCanStartAsyncTasks;
	}

	void SetRewindData(FRewindData* RewindData)
	{
		MRewindData = RewindData;
	}

	FRewindData* GetRewindData()
	{
		return MRewindData;
	}

	CHAOS_API void DisableParticleWithRemovalEvent(FGeometryParticleHandle* Particle);
	const TArray<FRemovalData>& GetAllRemovals() { return MAllRemovals; }
	void ResetAllRemovals() { MAllRemovals.Reset(); }

	virtual void SetName(const FString& InName) { EvolutionName = FSharedDebugName(InName); }
	const FString& GetName() const { return EvolutionName.Value(); }

	// Internal used only 
	UE_INTERNAL virtual void WaitIntegrationComplete() { check(false); };

protected:
	int32 NumConstraints() const
	{
		int32 NumConstraints = 0;
		for (const FPBDConstraintContainer* Container : ConstraintContainers)
		{
			NumConstraints += Container->GetNumConstraints();
		}
		return NumConstraints;
	}

public:
	template <bool bPersistent>
	FORCEINLINE_DEBUGGABLE void RemoveParticleFromAccelerationStructure(TGeometryParticleHandleImp<FReal, 3, bPersistent>& ParticleHandle)
	{
		//TODO: at the moment we don't distinguish between the first time a particle is created and when it's just moved
		// If we had this distinction we could simply remove the entry for the async queue
		const FUniqueIdx UniqueIdx = ParticleHandle.UniqueIdx();
		FPendingSpatialData& SpatialData = AsyncAccelerationQueue.FindOrAdd(UniqueIdx, EPendingSpatialDataOperation::Delete);

		SpatialData.Operation = EPendingSpatialDataOperation::Delete;
		SpatialData.SpatialIdx = ParticleHandle.SpatialIdx();
		SpatialData.AccelerationHandle = FAccelerationStructureHandle(ParticleHandle);

		//Internal acceleration has all moves pending, so cancel them all now
		InternalAccelerationQueue.Remove(UniqueIdx);

		//remove particle immediately for intermediate structure
		//TODO: if we distinguished between first time adds we could avoid this. We could also make the RemoveElementFrom more strict and ensure when it fails
		InternalAcceleration->RemoveElementFrom(SpatialData.AccelerationHandle, SpatialData.SpatialIdx);
	}

	
protected:

	void UpdateConstraintPositionBasedState(FReal Dt)
	{
		// If any constraint container state depends on particle state, it gets updated here.
		// E.g., we can create constraints between close particles etc. Collision detection 
		// could be called from here, but currently is called explicitly elsewhere
		for (FPBDConstraintContainer* ConstraintContainer : ConstraintContainers)
		{
			ConstraintContainer->UpdatePositionBasedState(Dt);
		}
	}

	void CreateConstraintGraph()
	{
		// Update the current state of the graph based on existing particles and constraints.
		// Any new particles (from this tick) should have been added when they were enabled.
		IslandManager.UpdateParticles();

		// Add all constraints to the graph.
		// NOTE: in PersistentGraph mode, only new constraints need to be added and expired ones should be removed.
		// In non-peristent mode, all constraints in awake islands are removed every tick and so all would need to be re-added here.
		// @todo(chaos): it feels a bit inconsistent that particles are added when enabled, but constraints
		// are added here. Currently it needs to be this way to properly support user changes to sleep state
		// of particles, but we could probably make this cleaner.
		for (FPBDConstraintContainer* ConstraintContainer : ConstraintContainers)
		{
			ConstraintContainer->AddConstraintsToGraph(GetIslandManager());
		}
	}

	void CreateIslands()
	{
		// Package the constraints and particles into islands
		IslandManager.UpdateIslands();
	}

	void PruneInternalPendingData(FAccelerationStructure& AccelerationStructure);
	void FlushInternalAccelerationQueue();
	void FlushAsyncAccelerationQueue();
	void WaitOnAccelerationStructure();
	static void CopyUnBuiltDynamicAccelerationStructures(const TMap<FSpatialAccelerationIdx, TUniquePtr<FSpatialAccelerationCache>>& SpatialAccelerationCache, FAccelerationStructure* InternalAcceleration, FAccelerationStructure* AsyncInternalAcceleration, FAccelerationStructure* AsyncExternalAcceleration);
	static void CopyPristineAccelerationStructures(const TMap<FSpatialAccelerationIdx, TUniquePtr<FSpatialAccelerationCache>>& SpatialAccelerationCache, FAccelerationStructure* FromStructure, FAccelerationStructure* ToStructure, bool CheckPristine);

	TArray<FForceRule> ForceRules;
	TArray<FForceRule> ImpulseRules;
	FUpdatePositionRule ParticleUpdatePosition;
	FKinematicUpdateRule KinematicUpdate;
	FCaptureRewindRule CaptureRewindData;
	FCaptureRewindKinematicTargetRule CaptureRewindKinematicTarget;
	TArray<FPBDConstraintContainer*> ConstraintContainers;
	Private::FPBDIslandManager IslandManager;
	Private::FPBDIslandGroupManager IslandGroupManager;
	TArrayCollectionArray<TSerializablePtr<FChaosPhysicsMaterial>> PhysicsMaterials;
	TArrayCollectionArray<TUniquePtr<FChaosPhysicsMaterial>> PerParticlePhysicsMaterials;
	TArrayCollectionArray<bool> Collided;

	FPBDRigidsSOAs& Particles;
	THandleArray<FChaosPhysicsMaterial>& SolverPhysicsMaterials;
	FAccelerationStructure* InternalAcceleration;
	FAccelerationStructure* AsyncInternalAcceleration;
	FAccelerationStructure* AsyncExternalAcceleration;
	FRewindData* MRewindData = nullptr;

	//internal thread will push into this and external thread will consume
	TSpscQueue<FAccelerationStructure*> ExternalStructuresQueue;

	//external thread will push into this when done with structure
	//internal thread will pop from this to generate new structure
	TSpscQueue<FAccelerationStructure*> ExternalStructuresPool;

	//the backing buffer for all acceleration structures
	TArray<TUniquePtr<FAccelerationStructure>> AccelerationBackingBuffer;
	bool bIsSingleThreaded;

	// Allows us to tell evolution to stop starting async tasks if we are trying to cleanup solver/evo.
	bool bCanStartAsyncTasks;

	TArray<FUniqueIdx> UniqueIndicesPendingRelease;

	TArray<FRemovalData> MAllRemovals;
public:
	//The latest external timestamp we consumed inputs from, assigned to evolution when solver task executes, is used to stamp output data.
	int32 LatestExternalTimestampConsumed_Internal;	

protected:

	/** Pending operations for the internal acceleration structure */
	FPendingSpatialInternalDataQueue InternalAccelerationQueue;
	
	/** Pending operations for the acceleration structures being rebuilt asynchronously */
	FPendingSpatialDataQueue AsyncAccelerationQueue;

	Private::FTaskDispatcherEvolution TaskDispatcher;
	

	/*void SerializePendingMap(FChaosArchive& Ar, TMap<FGeometryParticleHandle*, FPendingSpatialData>& Map)
	{
		TArray<FGeometryParticleHandle*> Keys;
		if (!Ar.IsLoading())
		{
			Map.GenerateKeyArray(Keys);
		}
		Ar << AsAlwaysSerializableArray(Keys);
		for (auto Key : Keys)
		{
			FPendingSpatialData& PendingData = Map.FindOrAdd(Key);
			PendingData.Serialize(Ar);
		}
		//TODO: fix serialization
	}*/

	/** Used for async acceleration rebuild */
	TArrayAsMap<FUniqueIdx, uint32> ParticleToCacheInnerIdx;

	TMap<FSpatialAccelerationIdx, TUniquePtr<FSpatialAccelerationCache>> SpatialAccelerationCache;

	FORCEINLINE_DEBUGGABLE void ApplyParticlePendingData(const FPendingSpatialData& PendingData, FAccelerationStructure& SpatialAcceleration, bool bUpdateCache, bool bUpdateDynamicTrees);

	class FChaosAccelerationStructureTask
	{
	public:
		FChaosAccelerationStructureTask(ISpatialAccelerationCollectionFactory& InSpatialCollectionFactory
			, const TMap<FSpatialAccelerationIdx, TUniquePtr<FSpatialAccelerationCache>>& InSpatialAccelerationCache
			, FAccelerationStructure* InInternalAccelerationStructure
			, FAccelerationStructure* InExternalAccelerationStructure
			, bool InForceFullBuild
			, bool InIsSingleThreaded
			, bool bNeedsReset
			, std::atomic<bool>** bOutStarted
			, std::atomic<bool>** bOutKillTask);
		static FORCEINLINE TStatId GetStatId();
		static FORCEINLINE ENamedThreads::Type GetDesiredThread();
		static FORCEINLINE ESubsequentsMode::Type GetSubsequentsMode();
		void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent);

		ISpatialAccelerationCollectionFactory& SpatialCollectionFactory;
		const TMap<FSpatialAccelerationIdx, TUniquePtr<FSpatialAccelerationCache>>& SpatialAccelerationCache;
		FAccelerationStructure* InternalStructure;
		FAccelerationStructure* ExternalStructure;
		bool IsForceFullBuild;
		bool bIsSingleThreaded;
		bool bNeedsReset;
		std::atomic<bool> bStarted;
		std::atomic<bool> bKillTask;

	private:
		void UpdateStructure(FAccelerationStructure* AccelerationStructure, FAccelerationStructure* CopyToAccelerationStructure = nullptr);
	};
	FGraphEventRef AccelerationStructureTaskComplete;
	std::atomic<bool>* bAccelerationStructureTaskStarted;
	std::atomic<bool>* bAccelerationStructureTaskSignalKill;

	TUniquePtr<ISpatialAccelerationCollectionFactory> SpatialCollectionFactory;

	FAccelerationStructure* GetFreeSpatialAcceleration_Internal();
	void FreeSpatialAcceleration_External(FAccelerationStructure* Structure);

	void ReleaseIdx(FUniqueIdx Idx);
	void ReleasePendingIndices();

	TArray<FUniqueIdx> PendingReleaseIndices;	//for now just assume a one frame delay, but may need something more general
	bool bIsResim = false; 
	bool bIsReset = false;

	// Useful name for debugging. E.g., Indicates whether we are on client or server
	FSharedDebugName EvolutionName;

#if CHAOS_EVOLUTION_COLLISION_TESTMODE
	// Test Mode for Collision issues (resets particle positions every tick for repeatable testing)
	CHAOS_API void TestModeStep();
	CHAOS_API void TestModeParticleDisabled(FGeometryParticleHandle* Particle);
	CHAOS_API void TestModeSaveParticles();
	CHAOS_API void TestModeSaveParticle(FGeometryParticleHandle* Particle);
	CHAOS_API void TestModeUpdateSavedParticle(FGeometryParticleHandle* Particle);
	CHAOS_API void TestModeRestoreParticles();
	CHAOS_API void TestModeRestoreParticle(FGeometryParticleHandle* Particle);

	struct FTestModeParticleData
	{
		FVec3 X, P, V, W;
		FRotation3 R, Q;
	};
	TMap<FPBDRigidParticleHandle*, FTestModeParticleData> TestModeData;
#endif
};


}
