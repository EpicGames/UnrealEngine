// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

namespace Chaos
{
	class FPBDRigidsEvolutionBase;

	class FPBDRigidsEvolutionGBF;

	//The default evolution used by unreal
	using FPBDRigidsEvolution = FPBDRigidsEvolutionGBF;

	class FPBDRigidsSolver;

	using FPBDRigidsSolver = FPBDRigidsSolver;

	class FEventManager;

	using FEventManager = FEventManager;
}

class FGeometryCollectionPhysicsProxy;