// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/ChaosPerfTest.h"
#include "Chaos/Character/CharacterGroundConstraintContainer.h"
#include "Chaos/Collision/SpatialAccelerationBroadPhase.h"
#include "Chaos/Collision/SpatialAccelerationCollisionDetector.h"
#include "Chaos/Evolution/SolverBodyContainer.h"
#include "Chaos/PBDCollisionConstraints.h"
#include "Chaos/PBDRigidClustering.h"
#include "Chaos/PBDRigidsEvolution.h"
#include "Chaos/PerParticleAddImpulses.h"
#include "Chaos/PerParticleEtherDrag.h"
#include "Chaos/PerParticleEulerStepVelocity.h"
#include "Chaos/PerParticleExternalForces.h"
#include "Chaos/PerParticleGravity.h"
#include "Chaos/PerParticleInitForce.h"
#include "Chaos/PerParticlePBDEulerStep.h"
#include "Chaos/CCDUtilities.h"
#include "Chaos/Particle/ParticleUtilities.h"
#include "Chaos/PBDSuspensionConstraints.h"
#include "Chaos/ChaosDebugDraw.h"

namespace Chaos
{
	class FChaosArchive;
	class IResimCacheBase;
	class FEvolutionResimCache;

	CHAOS_API extern int32 AccelerationStructureUseDynamicTree;

	namespace CVars
	{
		CHAOS_API extern FRealSingle HackMaxAngularVelocity;
		CHAOS_API extern FRealSingle HackMaxVelocity;
		CHAOS_API extern FRealSingle SmoothedPositionLerpRate;
		CHAOS_API extern bool bChaosCollisionCCDUseTightBoundingBox;
		CHAOS_API extern int32 ChaosCollisionCCDConstraintMaxProcessCount;
		CHAOS_API extern int32 ChaosSolverDrawCCDThresholds;
	}

	using FPBDRigidsEvolutionCallback = TFunction<void(FReal Dt)>;

	using FPBDRigidsEvolutionIslandCallback = TFunction<void(int32 Island)>;

	using FPBDRigidsEvolutionInternalHandleCallback = TFunction<void(
		const FGeometryParticleHandle* OldParticle,
		FGeometryParticleHandle* NewParticle)>;

	class FPBDRigidsEvolutionGBF : public FPBDRigidsEvolutionBase
	{
	public:
		using Base = FPBDRigidsEvolutionBase;

		using FGravityForces = FPerParticleGravity;
		using FCollisionConstraints = FPBDCollisionConstraints;
		using FCollisionDetector = FSpatialAccelerationCollisionDetector;
		using FExternalForces = FPerParticleExternalForces;
		using FJointConstraints = FPBDJointCombinedConstraints;

		// Default settings for FChaosSolverConfiguration
		static constexpr int32 DefaultNumPositionIterations = 8;
		static constexpr int32 DefaultNumVelocityIterations = 2;
		static constexpr int32 DefaultNumProjectionIterations = 1;
		static constexpr FRealSingle DefaultCollisionMarginFraction = 0.05f;
		static constexpr FRealSingle DefaultCollisionMarginMax = 10.0f;
		static constexpr FRealSingle DefaultCollisionCullDistance = 3.0f;
		static constexpr FRealSingle DefaultCollisionMaxPushOutVelocity = 1000.0f;
		static constexpr FRealSingle DefaultCollisionDepenetrationVelocity = -1.0f;
		static constexpr int32 DefaultRestitutionThreshold = 1000;

		CHAOS_API FPBDRigidsEvolutionGBF(
			FPBDRigidsSOAs& InParticles, 
			THandleArray<FChaosPhysicsMaterial>& SolverPhysicsMaterials, 
			const TArray<ISimCallbackObject*>* InMidPhaseModifiers = nullptr,
			const TArray<ISimCallbackObject*>* InCCDModifiers = nullptr,
			const TArray<ISimCallbackObject*>* InStrainModifiers = nullptr,
			const TArray<ISimCallbackObject*>* InCollisionModifiers = nullptr,
			bool InIsSingleThreaded = false);
		CHAOS_API ~FPBDRigidsEvolutionGBF();

		virtual void SetName(const FString& InName)
		{ 
			FPBDRigidsEvolutionBase::SetName(InName);
			CollisionConstraints.SetName(EvolutionName);
		}

		void SetPreIntegrateCallback(const FPBDRigidsEvolutionCallback& Cb)
		{
			PreIntegrateCallback = Cb;
		}

		void SetPostIntegrateCallback(const FPBDRigidsEvolutionCallback& Cb)
		{
			PostIntegrateCallback = Cb;
		}

		void SetPreSolveCallback(const FPBDRigidsEvolutionCallback& Cb)
		{
			PreSolveCallback = Cb;
		}

		void SetPostSolveCallback(const FPBDRigidsEvolutionCallback& Cb)
		{
			PostSolveCallback = Cb;
		}

		void SetPostDetectCollisionsCallback(const FPBDRigidsEvolutionCallback& Cb)
		{
			PostDetectCollisionsCallback = Cb;
		}

		UE_DEPRECATED(5.4, "Use SetPreSolveCallback")
		void SetPreApplyCallback(const FPBDRigidsEvolutionCallback& Cb) { SetPreSolveCallback(Cb); }

		FORCEINLINE void SetInternalParticleInitilizationFunction(const FPBDRigidsEvolutionInternalHandleCallback& Cb)
		{ 
			InternalParticleInitilization = Cb;
		}

		FORCEINLINE void DoInternalParticleInitilization(const FGeometryParticleHandle* OldParticle, FGeometryParticleHandle* NewParticle) 
		{ 
			if(InternalParticleInitilization)
			{
				InternalParticleInitilization(OldParticle, NewParticle);
			}
		}

		void SetIsDeterministic(const bool bInIsDeterministic);

		void SetShockPropagationIterations(const int32 InPositionIts, const int32 InVelocityIts);

		CHAOS_API void Advance(const FReal Dt, const FReal MaxStepDt, const int32 MaxSteps);
		CHAOS_API void AdvanceOneTimeStep(const FReal dt, const FSubStepInfo& SubStepInfo = FSubStepInfo());

		FORCEINLINE FCollisionConstraints& GetCollisionConstraints() { return CollisionConstraints; }
		FORCEINLINE const FCollisionConstraints& GetCollisionConstraints() const { return CollisionConstraints; }

		FORCEINLINE FCollisionDetector& GetCollisionDetector() { return CollisionDetector; }
		FORCEINLINE const FCollisionDetector& GetCollisionDetector() const { return CollisionDetector; }

		FORCEINLINE FGravityForces& GetGravityForces() { return GravityForces; }
		FORCEINLINE const FGravityForces& GetGravityForces() const { return GravityForces; }

		FORCEINLINE const FRigidClustering& GetRigidClustering() const { return Clustering; }
		FORCEINLINE FRigidClustering& GetRigidClustering() { return Clustering; }

		UE_DEPRECATED(5.5, "No longer used, see combined constraint in FPBDRigidsEvolutionGBF::GetJointCombinedConstraints")
		FORCEINLINE FPBDJointConstraints& GetJointConstraints() { return JointConstraints.LinearConstraints; }
		UE_DEPRECATED(5.5, "No longer used, see combined constraint in FPBDRigidsEvolutionGBF::GetJointCombinedConstraints")
		FORCEINLINE const FPBDJointConstraints& GetJointConstraints() const { return JointConstraints.LinearConstraints; }

		FORCEINLINE FJointConstraints& GetJointCombinedConstraints() { return JointConstraints; }
		FORCEINLINE const FJointConstraints& GetJointCombinedConstraints() const { return JointConstraints; }

		FORCEINLINE FPBDSuspensionConstraints& GetSuspensionConstraints() { return SuspensionConstraints; }
		FORCEINLINE const FPBDSuspensionConstraints& GetSuspensionConstraints() const { return SuspensionConstraints; }

		FORCEINLINE FCharacterGroundConstraintContainer& GetCharacterGroundConstraints() { return CharacterGroundConstraints; }
		FORCEINLINE const FCharacterGroundConstraintContainer& GetCharacterGroundConstraints() const { return CharacterGroundConstraints; }


		//
		// Particle API (most of the particle API is in the base class)
		//

		/**
		 * User has moved a particle
		 * Does not change velocity.
		 * Will wake the particle if this is a move (i.e., bIsTeleport is false)
		 */
		CHAOS_API void SetParticleTransform(FGeometryParticleHandle* InParticle, const FVec3& InPos, const FRotation3& InRot, const bool bIsTeleport, const bool bWakeUp = true);

		/**
		 * Move a particle to a new location with a sweep and stop and the first opposing contact.
		 * Does not change velocity.
		 * Will wake the particle if this is a move (i.e., bIsTeleport is false)
		 */
		CHAOS_API virtual void SetParticleTransformSwept(FGeometryParticleHandle* InParticle, const FVec3& InPos, const FRotation3& InRot, const bool bIsTeleport);

		/**
		* Set the kinematic target for a particle. This will exist for only one tick - a new target must be set for the next tick if required.
		* If called on a dynamic object, is equivalent to SetParticleTransform with bIsTeleport=false
		* Returns true if the particle has been added to the moving kinematic particles list or the the active list.
		*/
		CHAOS_API bool SetParticleKinematicTarget(FGeometryParticleHandle* ParticleHandle, const FKinematicTarget& NewKinematicTarget);

		/*
		 * [EXPERIMENTAL] Apply a momentumless correction to the particle transform, usually as a result of a server correction.
		 * This will shift the particle by the supplied delta and handle updating of friction anchors or anything else that might prevent or undo the shift.
		 * If bApplyToConnectedBodies is true, any particle attached by a joint with locked linear limits will also get moved.
		 * NOTE: must be called prior to Integrate() to be effective.
		 * NOTE: be careful with bApplyToConnectedBodies - only one particle in the connected graph should have ApplyParticleTransformCorrectionDelta called on 
		 * it, otherwise you will get multiple particles trying to recorrect each other leading to very strange behaviour.
		 * Use @param ExcludeConnections to exclude particles from being corrected, this includes particles connected to the excluded particle as long as they are not connected to a non-excluded particle.
		 */
		CHAOS_API void ApplyParticleTransformCorrectionDelta(FGeometryParticleHandle* InParticle, const FVec3& InPosDelta, const FVec3& InRotDelta, const bool bApplyToConnectedBodies, const bool bInRecalculateFrictionOnConnectedBodies = true, const TArray<FParticleID>& ExcludeConnections = TArray<FParticleID>());

		/*
		 * [EXPERIMENTAL] Similar to ApplyParticleTransformCorrectionDelta, but supplied an absolute transform to jump to. This is used for snaps.
		 */
		CHAOS_API void ApplyParticleTransformCorrection(FGeometryParticleHandle* InParticle, const FVec3& InPos, const FRotation3& InRot, const bool bApplyToConnectedBodies, const bool bInRecalculateFrictionOnConnectedBodies = true, const TArray<FParticleID>& ExcludeConnections = TArray<FParticleID>());

		/*
		* [EXPERIMENTAL] Apply sleep state on connected particles that are dynamic
		*/
		CHAOS_API void ApplySleepOnConnectedParticles(FGeometryParticleHandle* InParticle);

		/**
		 * Called when a particle is moved. We need to reset some friction properties, sleeping properties, etc
		 */
		CHAOS_API void OnParticleMoved(FGeometryParticleHandle* InParticle, const FVec3& PrevX, const FRotation3& PrevR, const bool bIsTeleport, const bool bWakeUp = true);

		/**
		 * User has changed particle velocity or angular velocity
		 */
		CHAOS_API void SetParticleVelocities(FGeometryParticleHandle* InParticle, const FVec3& InV, const FVec3f& InW);

		/**
		 * User has changed particle dynamics
		 */
		CHAOS_API void SetParticleDynamics(FPBDRigidParticleHandle* Rigid, const FParticleDynamics& Dynamics);

		/**
		 * User has changed particle mass properties
		 */
		CHAOS_API void SetParticleMassProps(FPBDRigidParticleHandle* Rigid, const FParticleMassProps& MassProperties);

		/**
		 * Reload the particles cache for all particles where appropriate
		 */
		void ReloadParticlesCache();

		void DestroyParticleCollisionsInAllocator(FGeometryParticleHandle* Particle);

		virtual void DestroyTransientConstraints(FGeometryParticleHandle* Particle) override final;
		virtual void DestroyTransientConstraints() override final;

		/** Reset the collisions warm starting when resimulate. Ideally we should store
		  that in the RewindData history but probably too expensive for now */
		virtual void ResetCollisions() override;

		inline void EndFrame(FReal Dt)
		{
			Particles.GetNonDisabledDynamicView().ParallelFor([&](auto& Particle, int32 Index) {
				Particle.Acceleration() = FVec3(0);
				Particle.AngularAcceleration() = FVec3(0);
			});
		}

		// Called when a the material changes one or more shapes on a particle. Required because collisions cache material properties
		void ParticleMaterialChanged(FGeometryParticleHandle* Particle);

		CHAOS_API const FChaosPhysicsMaterial* GetFirstClusteredPhysicsMaterial(const FGeometryParticleHandle* Particle) const;

		CHAOS_API void Integrate(FReal Dt);

		CHAOS_API virtual void ApplyKinematicTargets(const FReal Dt, const FReal StepFraction) override final;

		CHAOS_API void Serialize(FChaosArchive& Ar);

		CHAOS_API TUniquePtr<IResimCacheBase> CreateExternalResimCache() const;
		CHAOS_API void SetCurrentStepResimCache(IResimCacheBase* InCurrentStepResimCache);

		FSpatialAccelerationBroadPhase& GetBroadPhase() { return BroadPhase; }

		// Create a joint constraint connecting two particles
		CHAOS_API FPBDJointConstraintHandle* CreateJointConstraint(const FParticlePair& InParticlePair, const FPBDJointSettings& InJointSettings);

		// Destroy a joint constraint
		CHAOS_API void DestroyJointConstraint(FPBDJointConstraintHandle* InJointHandle);

		// Change the settings of a joint constraint.
		// NOTE: May recreate the joint which would change the handle pointer. (Currently only when changing linear/non-linear solver type)
		CHAOS_API FPBDJointConstraintHandle* SetJointConstraintSettings(FPBDJointConstraintHandle* InJointHandle, const FPBDJointSettings& InJointSettings);

		CHAOS_API void TransferJointConstraintCollisions();

		// Resets VSmooth value to something plausible based on external forces to prevent object from going back to sleep if it was just impulsed.
		template <bool bPersistent>
		void ResetVSmoothFromForces(TPBDRigidParticleHandleImp<FReal, 3, bPersistent>& Particle)
		{
			const FReal SmoothRate = FMath::Clamp(CVars::SmoothedPositionLerpRate, 0.0f, 1.0f);
	
			// Reset VSmooth to something roughly in the same direction as what V will be after integration.
			// This is temp fix, if this is only re-computed after solve, island will get incorrectly put back to sleep even if it was just impulsed.
			FReal FakeDT = (FReal)1. / (FReal)30.;
			if (Particle.LinearImpulseVelocity().IsNearlyZero() == false || Particle.Acceleration().IsNearlyZero() == false)
			{
				const FVec3 PredictedLinearVelocity = Particle.GetV() + Particle.Acceleration() * FakeDT + Particle.LinearImpulseVelocity();
				Particle.VSmooth() =FMath::Lerp(Particle.VSmooth(), PredictedLinearVelocity, SmoothRate);
			}
			if (Particle.AngularImpulseVelocity().IsNearlyZero() == false || Particle.AngularAcceleration().IsNearlyZero() == false)
			{
				const FVec3 PredictedAngularVelocity = Particle.GetW() + Particle.AngularAcceleration() * FakeDT + Particle.AngularImpulseVelocity();
				Particle.WSmooth() = FMath::Lerp(Particle.WSmooth(), PredictedAngularVelocity, SmoothRate);
			}
		}

		template<typename TParticleView> 
		UE_DEPRECATED(5.4, "Use Integrate(Dt)")
		void Integrate(const TParticleView& InParticles, FReal Dt) { Integrate(Dt); }

		// Internal used only
		UE_INTERNAL virtual void WaitIntegrationComplete() override;

	protected:

		CHAOS_API void AdvanceOneTimeStepImpl(const FReal Dt, const FSubStepInfo& SubStepInfo);

		void ApplyMidPhaseModifier(FReal Dt);

		// Update the particle transform and fix collision anchors (used by client corrections)
		void ApplyParticleTransformCorrectionImpl(FGeometryParticleHandle* InParticle, const FRigidTransform3& InTransform, const bool bInRecalculateFriction = true);

		/* Get all the particles that are connected to InParticle by a joint with locked position limits
		 * Use @param ExcludeConnections to exclude particles, this includes particles connected to the excluded particle as long as they are not connected to a non-excluded particle. */
		TArray<FGeometryParticleHandle*> GetConnectedParticles(FGeometryParticleHandle* InParticle, const TArray<FParticleID>& ExcludeConnections = TArray<FParticleID>());

		void UpdateInertiaConditioning();

		FEvolutionResimCache* GetCurrentStepResimCache()
		{
			return CurrentStepResimCacheImp;
		}

		void UpdateCollisionSolverType();

		template <typename LambdaType>
		void DispatchKinematicTargetsTasks(int32 DispatchBatchIndex, int32 NumParticles, FReal Dt, FReal StepFraction, bool IsLastStep, LambdaType ReturnHandle);

		FRigidClustering Clustering;

		FPBDJointCombinedConstraints JointConstraints;
		FPBDSuspensionConstraints SuspensionConstraints;
		FCharacterGroundConstraintContainer CharacterGroundConstraints;

		FGravityForces GravityForces;
		FCollisionConstraints CollisionConstraints;
		FSpatialAccelerationBroadPhase BroadPhase;
		FSpatialAccelerationCollisionDetector CollisionDetector;

		FPBDRigidsEvolutionCallback PreIntegrateCallback;
		FPBDRigidsEvolutionCallback PostIntegrateCallback;
		FPBDRigidsEvolutionCallback PostDetectCollisionsCallback;
		FPBDRigidsEvolutionCallback PreSolveCallback;
		FPBDRigidsEvolutionCallback PostSolveCallback;
		FPBDRigidsEvolutionInternalHandleCallback InternalParticleInitilization;
		FEvolutionResimCache* CurrentStepResimCacheImp;

		// @todo(chaos): evolution and collision constraints should not know about ISimCallbackObject. Fix this.
		const TArray<ISimCallbackObject*>* MidPhaseModifiers;
		const TArray<ISimCallbackObject*>* CCDModifiers;
		const TArray<ISimCallbackObject*>* CollisionModifiers;

		FCCDManager CCDManager;

		bool bIsDeterministic;

		// Used to track whether we have applied the latest Inertia Conditioning Cvars
		int32 InertiaConditioningEpoch;

#if CHAOS_EVOLUTION_COLLISION_TESTMODE
		void TestModeResetCollisions();
#endif

#if CHAOS_DEBUG_DRAW
	public:
		void SetDebugDrawScene(const ChaosDD::Private::FChaosDDScenePtr& InCDDScene);

	private:
		ChaosDD::Private::FChaosDDScenePtr CDDScene;
		ChaosDD::Private::FChaosDDTimelinePtr CDDTickTimeline;
#endif
	};

}

