// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/ParticleHandle.h"
#include "ChaosVisualDebugger/ChaosVisualDebuggerTrace.h"
#include "UObject/ExternalPhysicsCustomObjectVersion.h"

namespace Chaos
{
namespace CVars
{
	extern CHAOS_API bool bRemoveParticleFromMovingKinematicsOnDisable;
	extern CHAOS_API bool bChaosSolverCheckParticleViews;
}
	
namespace Private
{
	// The goal of that function is to reorder a particle view per category in place without reallocation in O(N). 
	// The different categories should not overlap (example : kinematic vs dynamic vs static...)
	template<typename CategoryFunc>
	FORCEINLINE void ReorderParticleView(TArrayView<FPBDRigidParticleHandle*>& ParticlesView, const CategoryFunc& CategoryTrigger, int32& CategoryOffset)
	{		
		const int32 NumParticles = ParticlesView.Num();
		for (int32 ParticleIndex = CategoryOffset; ParticleIndex < NumParticles; ++ParticleIndex)
		{
			if (CategoryTrigger(ParticlesView[ParticleIndex]))
			{
				if (ParticleIndex != CategoryOffset)
				{
					Swap(ParticlesView[ParticleIndex], ParticlesView[CategoryOffset]);
				}
				++CategoryOffset;
			}
		}
	}
}


// A helper function to get the debug name for a particle or "UNKNOWN" when debug names are unavailable
template<typename TParticleType>
const FString& GetParticleDebugName(const TParticleType& Particle)
{
#if CHAOS_DEBUG_NAME
	return *Particle.DebugName();
#else
	static const FString SNoDebugNames = "<UNKNOWN>";
	return SNoDebugNames;
#endif
}

class IParticleUniqueIndices
{
public:
	virtual ~IParticleUniqueIndices() = default;
	virtual FUniqueIdx GenerateUniqueIdx() = 0;
	virtual void ReleaseIdx(FUniqueIdx Unique) = 0;
};

class FParticleUniqueIndicesMultithreaded: public IParticleUniqueIndices
{
public:
	FParticleUniqueIndicesMultithreaded()
		: Block(0)
	{
		//Note: tune this so that all allocation is done at initialization
		AddPageAndAcquireNextId(/*bAcquireNextId = */ false);
	}

	FUniqueIdx GenerateUniqueIdx() override
	{
		while(true)
		{
			if(FUniqueIdx* Idx = FreeIndices.Pop())
			{
				return *Idx;
			}

			//nothing available so try to add some

			if(FPlatformAtomics::InterlockedCompareExchange(&Block,1,0) == 0)
			{
				//we got here first so add a new page
				FUniqueIdx RetIdx = FUniqueIdx(AddPageAndAcquireNextId(/*bAcquireNextId =*/ true));

				//release blocker. Note: I don't think this can ever fail, but no real harm in the while loop
				while(FPlatformAtomics::InterlockedCompareExchange(&Block,0,1) != 1)
				{
				}

				return RetIdx;
			}
		}
	}

	void ReleaseIdx(FUniqueIdx Unique) override
	{
		ensure(Unique.IsValid());
		int32 PageIdx = Unique.Idx / IndicesPerPage;
		int32 Entry = Unique.Idx % IndicesPerPage;
		FUniqueIdxPage& Page = *Pages[PageIdx];
		FreeIndices.Push(&Page.Indices[Entry]);
	}

	~FParticleUniqueIndicesMultithreaded()
	{
		//Make sure queue is empty, memory management of actual pages is handled automatically by TUniquePtr
		while(FreeIndices.Pop())
		{
		}
	}

private:

	int32 AddPageAndAcquireNextId(bool bAcquireNextIdx)
	{
		//Note: this should never really be called post initialization
		TUniquePtr<FUniqueIdxPage> Page = MakeUnique<FUniqueIdxPage>();
		const int32 PageIdx = Pages.Num();
		int32 FirstIdxInPage = PageIdx * IndicesPerPage;
		Page->Indices[0] = FUniqueIdx(FirstIdxInPage);

		//If we acquire next id we avoid pushing it into the queue
		for(int32 Count = bAcquireNextIdx ? 1 : 0; Count < IndicesPerPage; Count++)
		{
			Page->Indices[Count] = FUniqueIdx(FirstIdxInPage + Count);
			FreeIndices.Push(&Page->Indices[Count]);
		}

		Pages.Emplace(MoveTemp(Page));
		return bAcquireNextIdx ? FirstIdxInPage : INDEX_NONE;
	}

	static constexpr int32 IndicesPerPage = 1024;
	struct FUniqueIdxPage
	{
		FUniqueIdx Indices[IndicesPerPage];
	};
	TArray<TUniquePtr<FUniqueIdxPage>> Pages;
	TLockFreePointerListFIFO<FUniqueIdx,0> FreeIndices;
	volatile int8 Block;
};

using FParticleUniqueIndices = FParticleUniqueIndicesMultithreaded;

/**
* A list of particles held in a array.
* Also contains a particle-index map for faster find/removal with large lists.
* @see TParticleArray
*/
template <typename TParticleType>
class TParticleMapArray
{
public:
	TParticleMapArray()
		: ContainerListMask(EGeometryParticleListMask::None)
	{
	}

	void Reset()
	{
		ParticleToIndex.Reset();
		ParticleArray.Reset();
	}
	
	bool Contains(const TParticleType* Particle) const
	{
		return ParticleToIndex.Contains(Particle);
	}
	
	/** Check if the particle handle has a type compatible with the map array */
	bool HasValidType(FPBDRigidParticleHandle* ParticleHandle)
	{
		return ParticleHandle && (ParticleHandle->GetParticleType() >= TParticleType::StaticType());
	}
	
	/** Add a list of particles to the map array in O(k) */
	void Insert(const TArrayView<FPBDRigidParticleHandle*>& ParticlesToInsert)
	{
		if (ParticlesToInsert.IsEmpty())
		{
			return;
		}
		TArray<bool> Contains;
		Contains.AddZeroed(ParticlesToInsert.Num());

		// TODO: Compile time check ensuring TParticle2 is derived from TParticle1?
		int32 NextIdx = ParticleArray.Num();
		for (int32 Idx = 0; Idx < ParticlesToInsert.Num(); ++Idx)
		{
			if (HasValidType(ParticlesToInsert[Idx]))
			{
				TParticleType* Particle = static_cast<TParticleType*>(ParticlesToInsert[Idx]);
				Contains[Idx] = ParticleToIndex.Contains(Particle);
				if (!Contains[Idx])
				{
					ParticleToIndex.Add(Particle, NextIdx++);
				}
			}
		}
		ParticleArray.Reserve(ParticleArray.Num() + NextIdx - ParticleArray.Num());
		for (int32 Idx = 0; Idx < ParticlesToInsert.Num(); ++Idx)
		{
			if (!Contains[Idx] && HasValidType(ParticlesToInsert[Idx]))
			{
				TParticleType* Particle = static_cast<TParticleType*>(ParticlesToInsert[Idx]);
				ParticleArray.Add(Particle);
			}
		}
	}

	void Insert(TParticleType* Particle)
	{
		if (ParticleToIndex.Contains(Particle) == false)
		{
			ParticleToIndex.Add(Particle, ParticleArray.Num());
			ParticleArray.Add(Particle);
		}
	}

	/** Remove a list of particles from the map array in O(k) (Find, RemoveAtSwap and Remove are O(1)) */
	void Remove(const TArrayView<FPBDRigidParticleHandle*>& ParticlesToRemove)
	{
		if (ParticlesToRemove.IsEmpty())
		{
			return;
		}
		for (int32 Idx = 0; Idx < ParticlesToRemove.Num(); ++Idx)
		{
			if (HasValidType(ParticlesToRemove[Idx]))
			{
				const TParticleType* Particle = static_cast<TParticleType*>(ParticlesToRemove[Idx]);
				if (int32* IdxPtr = ParticleToIndex.Find(Particle))
				{
					int32 ParticleIdx = *IdxPtr;
					ParticleArray.RemoveAtSwap(ParticleIdx, EAllowShrinking::No);
					if (ParticleIdx < ParticleArray.Num())
					{
						//update swapped element with new index
						ParticleToIndex[ParticleArray[ParticleIdx]] = ParticleIdx;
					}
					ParticleToIndex.Remove(Particle);
				}
			}
		}
		ParticleArray.Shrink();
	}

	void Remove(TParticleType* Particle)
	{
		if (int32* IdxPtr = ParticleToIndex.Find(Particle))
		{
			int32 Idx = *IdxPtr;
			ParticleArray.RemoveAtSwap(Idx);
			if (Idx < ParticleArray.Num())
			{
				//update swapped element with new index
				ParticleToIndex[ParticleArray[Idx]] = Idx;
			}
			ParticleToIndex.Remove(Particle);
		}
	}

	void Serialize(FChaosArchive& Ar)
	{
		TArray<TSerializablePtr<TParticleType>>& SerializableArray = AsAlwaysSerializableArray(ParticleArray);
		Ar << SerializableArray;

		int32 Idx = 0;
		for (auto Particle : ParticleArray)
		{
			ParticleToIndex.Add(Particle, Idx++);
		}
	}

	const TArray<TParticleType*>& GetArray() const { return ParticleArray;	}
	TArray<TParticleType*>& GetArray() { return ParticleArray; }

	void SetContainerListMask(const EGeometryParticleListMask InMask) { ContainerListMask = InMask; }
	EGeometryParticleListMask GetContainerListMask() const { return ContainerListMask; }

private:
	TMap<TParticleType*, int32> ParticleToIndex;
	TArray<TParticleType*> ParticleArray;
	EGeometryParticleListMask ContainerListMask;
};

/**
* A list of particles held in a array.
* O(N) find/remove so only for use on small lists. 
* @see TParticleArrayMap
*/
template <typename TParticleType>
class TParticleArray
{
public:
	TParticleArray()
		: ContainerListMask(EGeometryParticleListMask::None)
	{
	}

	void Reset()
	{
		ParticleArray.Reset();
	}

	bool Contains(const TParticleType* Particle) const
	{
		return ParticleArray.Contains(Particle);
	}

	void Insert(TParticleType* Particle)
	{
		ParticleArray.Add(Particle);
	}

	void Remove(TParticleType* Particle)
	{
		ParticleArray.Remove(Particle);
	}

	const TArray<TParticleType*>& GetArray() const { return ParticleArray; }
	TArray<TParticleType*>& GetArray() { return ParticleArray; }

	void SetContainerListMask(const EGeometryParticleListMask InMask) { ContainerListMask = InMask; }
	EGeometryParticleListMask GetContainerListMask() const { return ContainerListMask; }

private:
	TArray<TParticleType*> ParticleArray;
	EGeometryParticleListMask ContainerListMask;
};

class FPBDRigidsSOAs
{
public:
	FPBDRigidsSOAs(IParticleUniqueIndices& InUniqueIndices)
		: UniqueIndices(InUniqueIndices)
	{
#if CHAOS_DETERMINISTIC
		BiggestParticleID = 0;
#endif

		StaticParticles = MakeUnique<FGeometryParticles>();
		StaticDisabledParticles = MakeUnique<FGeometryParticles>();

		KinematicParticles = MakeUnique<FKinematicGeometryParticles>();
		KinematicDisabledParticles = MakeUnique<FKinematicGeometryParticles>();

		DynamicDisabledParticles = MakeUnique<FPBDRigidParticles>();
		DynamicParticles = MakeUnique<FPBDRigidParticles>();
		DynamicKinematicParticles = MakeUnique<FPBDRigidParticles>();

		ClusteredParticles = MakeUnique<FPBDRigidClusteredParticles>();

		GeometryCollectionParticles = MakeUnique<TPBDGeometryCollectionParticles<FReal, 3>>();

		SetContainerListMasks();

		UpdateViews();
	}

	FPBDRigidsSOAs(const FPBDRigidsSOAs&) = delete;
	FPBDRigidsSOAs(FPBDRigidsSOAs&& Other) = delete;

	~FPBDRigidsSOAs()
	{
		// Abandonning the particles, don't worry about ordering anymore.
		ClusteredParticles->RemoveParticleBehavior() = ERemoveParticleBehavior::RemoveAtSwap;
		GeometryCollectionParticles->RemoveParticleBehavior() = ERemoveParticleBehavior::RemoveAtSwap;
	}

	void Reset()
	{
		check(0);
	}

	void ShrinkArrays(const float MaxSlackFraction, const int32 MinSlack)
	{
		StaticParticles->ShrinkArrays(MaxSlackFraction, MinSlack);
		StaticDisabledParticles->ShrinkArrays(MaxSlackFraction, MinSlack);

		KinematicParticles->ShrinkArrays(MaxSlackFraction, MinSlack);
		KinematicDisabledParticles->ShrinkArrays(MaxSlackFraction, MinSlack);

		DynamicDisabledParticles->ShrinkArrays(MaxSlackFraction, MinSlack);
		DynamicParticles->ShrinkArrays(MaxSlackFraction, MinSlack);
		DynamicKinematicParticles->ShrinkArrays(MaxSlackFraction, MinSlack);

		ClusteredParticles->ShrinkArrays(MaxSlackFraction, MinSlack);

		GeometryCollectionParticles->ShrinkArrays(MaxSlackFraction, MinSlack);
	}

	void UpdateDirtyViews()
	{
		// @todo(chaos): this should only refresh views that may have changed
		UpdateViews();
	}
	
	TArray<FGeometryParticleHandle*> CreateStaticParticles(int32 NumParticles, const FUniqueIdx* ExistingIndices = nullptr, const FGeometryParticleParameters& Params = FGeometryParticleParameters())
	{
		LLM_SCOPE_BYNAME(TEXT("Physics/StaticParticles"));

		auto Results = CreateParticlesHelper<FGeometryParticleHandle>(NumParticles, ExistingIndices, Params.bDisabled ? StaticDisabledParticles : StaticParticles, Params);
		UpdateViews();
		return Results;
	}
	TArray<FKinematicGeometryParticleHandle*> CreateKinematicParticles(int32 NumParticles, const FUniqueIdx* ExistingIndices = nullptr,  const FKinematicGeometryParticleParameters& Params = FKinematicGeometryParticleParameters())
	{
		LLM_SCOPE_BYNAME(TEXT("Physics/KinematicParticles"));

		auto Results = CreateParticlesHelper<FKinematicGeometryParticleHandle>(NumParticles, ExistingIndices, Params.bDisabled ? KinematicDisabledParticles : KinematicParticles, Params);
		UpdateViews();
		return Results;
	}
	TArray<FPBDRigidParticleHandle*> CreateDynamicParticles(int32 NumParticles, const FUniqueIdx* ExistingIndices = nullptr,  const FPBDRigidParticleParameters& Params = FPBDRigidParticleParameters())
	{
		LLM_SCOPE_BYNAME(TEXT("Physics/DynamicParticles"));

		auto Results = CreateParticlesHelper<FPBDRigidParticleHandle>(NumParticles, ExistingIndices, Params.bDisabled ? DynamicDisabledParticles : DynamicParticles, Params);;

		if (!Params.bStartSleeping)
		{
			AddToActiveArray(Results);
		}
		UpdateViews();
		return Results;
	}
	TArray<TPBDGeometryCollectionParticleHandle<FReal, 3>*> CreateGeometryCollectionParticles(int32 NumParticles, const FUniqueIdx* ExistingIndices = nullptr, const FPBDRigidParticleParameters& Params = FPBDRigidParticleParameters())
	{
		LLM_SCOPE_BYNAME(TEXT("Physics/GeometryCollectionParticles"));

		TArray<TPBDGeometryCollectionParticleHandle<FReal, 3>*> Results = CreateParticlesHelper<TPBDGeometryCollectionParticleHandle<FReal, 3>>(
			NumParticles, ExistingIndices, GeometryCollectionParticles, Params);
		for (auto* Handle : Results)
		{
			if (Params.bStartSleeping)
			{
				Handle->SetObjectStateLowLevel(Chaos::EObjectStateType::Sleeping);
				Handle->SetSleeping(true);
			}
			else
			{
				Handle->SetObjectStateLowLevel(Chaos::EObjectStateType::Dynamic);
				Handle->SetSleeping(false);
			}
			if (!Params.bDisabled)
			{
				InsertGeometryCollectionParticle(Handle);
			}
		}
		UpdateViews();
		
		return Results;
	}

	/** Used specifically by PBDRigidClustering. These have special properties for maintaining relative order, efficiently switching from kinematic to dynamic, disable to enable, etc... */
	TArray<FPBDRigidClusteredParticleHandle*> CreateClusteredParticles(int32 NumParticles, const FUniqueIdx* ExistingIndices = nullptr,  const FPBDRigidParticleParameters& Params = FPBDRigidParticleParameters())
	{
		LLM_SCOPE_BYNAME(TEXT("Physics/ClusteredParticles"));

		auto NewClustered = CreateParticlesHelper<FPBDRigidClusteredParticleHandle>(NumParticles, ExistingIndices, ClusteredParticles, Params);
		
		if (!Params.bDisabled)
		{
			InsertClusteredParticles(NewClustered);
		}

		if (!Params.bStartSleeping)
		{
			AddToActiveArray(reinterpret_cast<TArray<FPBDRigidParticleHandle*>&>(NewClustered));
		}

		UpdateViews();
		
		return NewClustered;
	}
	
	void ClearTransientDirty()
	{
		TransientDirtyMapArray.Reset();
	}
	void MarkTransientDirtyParticles(TArrayView<FPBDRigidParticleHandle*>& RigidParticles, const bool bUpdateViews = true)
	{
		if (!RigidParticles.IsEmpty())
		{
			int32 ParticleOffset = 0;
			
			// The particles in [0-ParticleOffset) will be the enabled ones switching to the new state
			Private::ReorderParticleView(RigidParticles, [this](FPBDRigidParticleHandle* RigidParticle) -> bool 
				{ return (!ActiveParticlesMapArray.Contains(RigidParticle) && !MovingKinematicsMapArray.Contains(RigidParticle));},ParticleOffset);	
		
			const int32 NumTransients = ParticleOffset;
			if (NumTransients > 0)
			{
				// Mark transient particles dirty
				TArrayView<FPBDRigidParticleHandle*> TransientParticles(&RigidParticles[0], NumTransients);
				AddToList(TransientParticles, TransientDirtyMapArray);
			
				for (FPBDRigidParticleHandle* Particle : TransientParticles)
				{
					CheckParticleViewMask(Particle);
				}
			
				if (bUpdateViews)
				{
					UpdateViews();
				}
			}
		}
	}
	

	// @todo(chaos): keep track of which views may be dirty and lazily update them
	// rather than calling UpdateViews all the time
	void MarkTransientDirtyParticle(FGeometryParticleHandle* GeometryParticle, const bool bUpdateViews = true)
	{
		if(FPBDRigidParticleHandle* RigidParticle =  GeometryParticle->CastToRigidParticle())
		{
			TArrayView<FPBDRigidParticleHandle*> RigidParticles(&RigidParticle, 1);
			MarkTransientDirtyParticles(RigidParticles, bUpdateViews);
		}
	}

	void DestroyParticle(FGeometryParticleHandle* Particle)
	{
		if (bResimulating)
		{
			RemoveFromList(Particle, ResimStaticParticles);
			FKinematicGeometryParticleHandle* Kinematic = Particle->CastToKinematicParticle();
			if (Kinematic)
			{
				RemoveFromList(Kinematic, ResimKinematicParticles);
			}
		}

		CVD_TRACE_PARTICLE_DESTROYED(Particle);

		auto PBDRigid = Particle->CastToRigidParticle();
		if(PBDRigid)
		{
			if (bResimulating)
			{
				RemoveFromList(PBDRigid, ResimDynamicParticles);
				RemoveFromList(PBDRigid, ResimDynamicKinematicParticles);
			}

			RemoveFromActiveArray(PBDRigid, /*bStillDirty=*/ false);
			RemoveFromList(PBDRigid, MovingKinematicsMapArray);

			if (TPBDGeometryCollectionParticleHandle<FReal, 3>* GCHandle = Particle->CastToGeometryCollection())
			{
				RemoveGeometryCollectionParticle(GCHandle);
			}
			else if (FPBDRigidClusteredParticleHandle* PBDRigidClustered = Particle->CastToClustered())
			{
				RemoveClusteredParticle(PBDRigidClustered);
			}

			// Check for sleep events referencing this particle
			// TODO think about this case more
			GetDynamicParticles().GetSleepDataLock().WriteLock();
			TArray<TSleepData<FReal, 3>>& SleepData = GetDynamicParticles().GetSleepData();

			SleepData.RemoveAllSwap([Particle](TSleepData<FReal, 3>& Entry) 
			{
				return Entry.Particle == Particle;
			});

			GetDynamicParticles().GetSleepDataLock().WriteUnlock();
		}

		ParticleHandles.DestroyHandleSwap(Particle);

		UpdateViews();
	}

	/**
	 * A disabled particle is ignored by the solver.
	 */
	void DisableParticle(FGeometryParticleHandle* Particle)
	{
		// Rigid particles express their disabled state with a boolean.
		// Disabled kinematic and static particles get shuffled to different SOAs.

		if (auto PBDRigid = Particle->CastToRigidParticle())
		{
			PBDRigid->SetDisabled(true);
			PBDRigid->SetVf(FVec3f(0));
			PBDRigid->SetWf(FVec3f(0));

			if (TPBDGeometryCollectionParticleHandle<FReal, 3>* PBDRigidGC = Particle->CastToGeometryCollection())
			{
				RemoveGeometryCollectionParticle(PBDRigidGC);
			}
			else if (FPBDRigidClusteredParticleHandle* PBDRigidClustered = Particle->CastToClustered())
			{
				RemoveClusteredParticle(PBDRigidClustered);
			}
			else
			{
				SetDynamicParticleSOA(PBDRigid);
			}

			// All active particles RIGID particles
			{
				RemoveFromActiveArray(PBDRigid, /*bStillDirty=*/false);
				if (CVars::bRemoveParticleFromMovingKinematicsOnDisable)
				{
					RemoveFromList(PBDRigid, MovingKinematicsMapArray);
				}
			}
		}
		else if (Particle->CastToKinematicParticle())
		{
			Particle->MoveToSOA(*KinematicDisabledParticles);
		}
		else
		{
			Particle->MoveToSOA(*StaticDisabledParticles);
		}

		CheckParticleViewMask(Particle);

		UpdateViews();
	}

	void EnableParticle(FGeometryParticleHandle* Particle)
	{
		// Rigid particles express their disabled state with a boolean.
		// Disabled kinematic and static particles get shuffled to differnt SOAs.

		if (auto PBDRigid = Particle->CastToRigidParticle())
		{
			PBDRigid->SetDisabled(false);
			// DisableParticle() zeros V and W.  We do nothing here and assume the client
			// sets appropriate values.

			if (TPBDGeometryCollectionParticleHandle<FReal, 3>* PBDRigidGC = Particle->CastToGeometryCollection())
			{
				RemoveGeometryCollectionParticle(PBDRigidGC);
				InsertGeometryCollectionParticle(PBDRigidGC);
			}
			else if (FPBDRigidClusteredParticleHandle* PBDRigidClustered = Particle->CastToClustered())
			{
				InsertClusteredParticle(PBDRigidClustered);
			}
			else
			{
				SetDynamicParticleSOA(PBDRigid);
			}

			if (!PBDRigid->Sleeping() && Particle->ObjectState() == EObjectStateType::Dynamic)
			{
				AddToActiveArray(PBDRigid);
			}
		}
		else if (Particle->CastToKinematicParticle())
		{
			Particle->MoveToSOA(*KinematicParticles);
		}
		else
		{
			Particle->MoveToSOA(*StaticParticles);
		}

		CheckParticleViewMask(Particle);

		UpdateViews();
	}

	/**
	 * Wake a sleeping dynamic non-disabled particle.
	 */
	void ActivateParticle(FGeometryParticleHandle* Particle, const bool DeferUpdateViews=false)
	{
		if (auto PBDRigid = Particle->CastToRigidParticle())
		{
			if (PBDRigid->ObjectState() == EObjectStateType::Sleeping ||
				PBDRigid->ObjectState() == EObjectStateType::Dynamic)
			{
				if (ensure(!PBDRigid->Disabled()))
				{
					// Sleeping state is currently expressed in 2 places...
					PBDRigid->SetSleeping(false);
					PBDRigid->SetObjectStateLowLevel(EObjectStateType::Dynamic);
		
					if (TPBDGeometryCollectionParticleHandle<FReal, 3>* PBDRigidGC = Particle->CastToGeometryCollection())
					{
						RemoveGeometryCollectionParticle(PBDRigidGC);
						InsertGeometryCollectionParticle(PBDRigidGC);
					}
					check(PBDRigid->ObjectState() == EObjectStateType::Dynamic);
					AddToActiveArray(PBDRigid);

					CheckParticleViewMask(Particle);

					if(!DeferUpdateViews)
					{
						UpdateViews();
					}
				}
			}
		}
	}

	/**
	 * Wake multiple dynamic non-disabled particles.
	 */
	void ActivateParticles(const TArray<FGeometryParticleHandle*>& Particles)
	{
		for (auto Particle : Particles)
		{
			ActivateParticle(Particle, true);
		}
		
		UpdateViews();
	}

	/**
	 * Put a non-disabled dynamic particle to sleep.
	 *
	 * If \p DeferUpdateViews is \c true, then it's assumed this function
	 * is being called in a loop and it won't update the SOA view arrays.
	 */
	void DeactivateParticle(
		FGeometryParticleHandle* Particle,
		const bool DeferUpdateViews=false)
	{
		if(auto PBDRigid = Particle->CastToRigidParticle())
		{
			if (PBDRigid->ObjectState() == EObjectStateType::Dynamic ||
				PBDRigid->ObjectState() == EObjectStateType::Sleeping)
			{
				if (ensure(!PBDRigid->Disabled()))
				{
					// Sleeping state is currently expressed in 2 places...
					PBDRigid->SetSleeping(true);
					PBDRigid->SetObjectStateLowLevel(EObjectStateType::Sleeping);

					if (TPBDGeometryCollectionParticleHandle<FReal, 3>* PBDRigidGC = Particle->CastToGeometryCollection())
					{
						RemoveGeometryCollectionParticle(PBDRigidGC);
						InsertGeometryCollectionParticle(PBDRigidGC);
					}
					RemoveFromActiveArray(PBDRigid, /*bStillDirty=*/true);

					CheckParticleViewMask(Particle);

					if (!DeferUpdateViews)
					{
						UpdateViews();
					}
				}
			}
		}
	}

	/**
	 * Put multiple dynamic non-disabled particles to sleep.
	 */
	void DeactivateParticles(const TArray<FGeometryParticleHandle*>& Particles)
	{
		for (auto Particle : Particles)
		{
			DeactivateParticle(Particle, true);
		}
		UpdateViews();
	}
	
	/**
	* Rebuild views if necessary.
	*/
	void RebuildViews()
	{
		UpdateViews();
	}

	template<typename T, int d>
	void MoveParticlesToSOA(TArrayView<FPBDRigidParticleHandle*>& RigidParticles, TGeometryParticles<T, d>& ToSOA)
	{
		if (!RigidParticles.IsEmpty() && RigidParticles[0])
		{
			int32 ParticleOffset = 0;
			TGeometryParticles<T, d>* FromSOA = RigidParticles[0]->GeometryParticles;
			
			// Only allocation in the SetObjectState pipeline but we need to transfer particles->indices anyway
			TArray<int32> ParticleIndices;
			ParticleIndices.Init(0, RigidParticles.Num());
			
			while (ParticleOffset < RigidParticles.Num())
			{
				const uint32 PreviousOffset = ParticleOffset;
			
				// Reorder all the particles that have the same SOA as the FromSOA among the remaining ones 
				// [ParticleOffset+1...RigidParticles.Num()] and push them at the front of the array
				Private::ReorderParticleView(RigidParticles, [&FromSOA](FPBDRigidParticleHandle* RigidParticle) -> bool
					{
						return RigidParticle && (RigidParticle->GeometryParticles == FromSOA);
					}, ParticleOffset);
				
				if (FromSOA != &ToSOA)
				{
					// Fill partially the particle indices that we want to move to the target SOA
					for (int32 ViewIndex = PreviousOffset; ViewIndex < ParticleOffset; ++ViewIndex)
					{
						ParticleIndices[ViewIndex] = RigidParticles[ViewIndex]->ParticleIdx;
					}
					// Build the particle indices view used to update the SOAs
					const int32 NumParticles = ParticleOffset - PreviousOffset;
					TArrayView<int32> IndicesView(&ParticleIndices[PreviousOffset], NumParticles);
					
					// For each SOA move the particles to the target SOA
					uint32 OffsetIndex = ToSOA.Size();
					FromSOA->MoveToOtherParticles(IndicesView, ToSOA);
					check(ToSOA.Size() == (OffsetIndex + NumParticles))
	
					// Update the particle indices for the removed/added particles
					for(int32 GeometryIndex : IndicesView)
					{
						// Particles that have been moved with the swap should be updated with the right index
						if (static_cast<uint32>(GeometryIndex) < FromSOA->Size())
						{
							FromSOA->Handle(GeometryIndex)->ParticleIdx = GeometryIndex;
						}
						// Particles SOA index and mask are updated
						if(OffsetIndex < ToSOA.Size())
						{ 
							ToSOA.Handle(OffsetIndex)->ParticleIdx = OffsetIndex;
							ToSOA.Handle(OffsetIndex)->GeometryParticles = &ToSOA;
							
							ToSOA.ListMask(OffsetIndex) &= ~FromSOA->GetContainerListMask();
							ToSOA.ListMask(OffsetIndex) |= ToSOA.GetContainerListMask();
						}
						++OffsetIndex;
					}
				}
				if (ParticleOffset < RigidParticles.Num())
				{
					FromSOA = RigidParticles[ParticleOffset]->GeometryParticles;
				}
			}
		}
	}
	
	/** Update active particles SOAs, list and views */
	void UpdateActiveParticlesSOA(TArrayView<FPBDRigidParticleHandle*>& RigidParticles, const EObjectStateType ObjectState, int32& ParticleOffset)
	{	
		ParticleOffset = 0;
		
		// The particles in [0-ParticleOffset) will be the disabled one whereas the [ParticleOffset,NumParticles) will be the enabled
		Private::ReorderParticleView(RigidParticles, [](FPBDRigidParticleHandle* RigidParticle) -> bool 
			{ return RigidParticle && RigidParticle->Disabled(); },ParticleOffset);
		
		const int32 NumDisabled = ParticleOffset;
		if (NumDisabled > 0)
		{
			TArrayView<FPBDRigidParticleHandle*> DisabledParticles(&RigidParticles[0], NumDisabled);
			
			RemoveFromActiveArray(DisabledParticles, /*bStillDirty=*/ false);
			if (CVars::bRemoveParticleFromMovingKinematicsOnDisable)
			{
				RemoveFromList(DisabledParticles, MovingKinematicsMapArray);
			}
		}
		
		const int32 NumEnabled= RigidParticles.Num()-ParticleOffset;
		if (NumEnabled > 0)
		{
			TArrayView<FPBDRigidParticleHandle*> EnabledParticles(&RigidParticles[ParticleOffset], NumEnabled);
			if (ObjectState != EObjectStateType::Dynamic)
			{
				RemoveFromActiveArray(EnabledParticles, /*bStillDirty=*/true);
			}
			else
			{
				AddToActiveArray(EnabledParticles);
			}

			if (ObjectState != EObjectStateType::Kinematic)
			{
				RemoveFromList(EnabledParticles, MovingKinematicsMapArray);
			}
		}
	}
	/** Function to be executed after the particles are moved to the dynamic/clustered SOAs */
	void PostMoveParticlesSOA(TArrayView<FPBDRigidParticleHandle*>& RigidParticles, const bool bUpdateViews)
	{
		// Check if the view mask is correct only for no shipping build
		for(FPBDRigidParticleHandle* RigidParticle : RigidParticles)
		{ 
			CheckParticleViewMask(RigidParticle);
		}

		// Update the views if necessary
		if(bUpdateViews)
		{ 
			UpdateViews();
		}
	}

	/** Update dynamic particles SOAs, list and views */
	void SetDynamicParticlesSOA(TArrayView<FPBDRigidParticleHandle*>& RigidParticles, const EObjectStateType ObjectState, const bool bUpdateViews)
	{
        int32 ParticleOffset = 0;
		
		// Update the active particles SOA
		UpdateActiveParticlesSOA(RigidParticles, ObjectState, ParticleOffset);
		
		const int32 NumDisabled = ParticleOffset;
		if (NumDisabled > 0)
		{
			// For disabled particles move them into the dynamic disabled SOA
			TArrayView<FPBDRigidParticleHandle*> DisabledParticles(&RigidParticles[0], NumDisabled);
			MoveParticlesToSOA(DisabledParticles, *DynamicDisabledParticles);
		}
		
		const int32 NumEnabled = RigidParticles.Num()-ParticleOffset;
		if (NumEnabled > 0)
		{
			TArrayView<FPBDRigidParticleHandle*> EnabledParticles(&RigidParticles[ParticleOffset], NumEnabled);
			// For enabled particles move to dynamic/kinematic SOA. No sleeping/static ones so default to dynamic.
			switch (ObjectState)
			{
				case EObjectStateType::Kinematic:
					MoveParticlesToSOA(EnabledParticles, *DynamicKinematicParticles);
					if (bResimulating)
					{
						RemoveFromList(EnabledParticles, ResimDynamicParticles);
						AddToList(EnabledParticles, ResimDynamicKinematicParticles);
					}
					break;

				case EObjectStateType::Dynamic:
				default : // TODO: Special SOAs for sleeping and static particles?
					MoveParticlesToSOA(EnabledParticles, *DynamicParticles);
					if (bResimulating)
					{
						RemoveFromList(EnabledParticles, ResimDynamicKinematicParticles);
						AddToList(EnabledParticles, ResimDynamicParticles);
					}
					break;
			}
		}
		PostMoveParticlesSOA(RigidParticles, bUpdateViews);
	}

	/** Update dynamic particle SOAs, list and views */
	void SetDynamicParticleSOA(FPBDRigidParticleHandle* RigidParticle)
	{
		if (RigidParticle)
		{
			TArrayView<FPBDRigidParticleHandle*> RigidParticles(&RigidParticle, 1);
			SetDynamicParticlesSOA(RigidParticles, RigidParticle->ObjectState(), true);
		}
	}
	
	template<typename SOAType>
	void UpdateClusteredParticlesSOA(TArrayView<FPBDRigidParticleHandle*>& RigidParticles, 
		const EObjectStateType ObjectState, SOAType& ClusteredSOA, int32& ParticleOffset)
	{
		MoveParticlesToSOA(RigidParticles, ClusteredSOA);

		if (bResimulating)
		{
			RemoveFromList(RigidParticles, ResimDynamicKinematicParticles);
			AddToList(RigidParticles, ResimDynamicParticles);
		}
		
		ParticleOffset = 0;
		UpdateActiveParticlesSOA(RigidParticles, ObjectState, ParticleOffset);
	}

	/** Update clustered particle SOAs, list and views */
	void SetClusteredParticlesSOA(TArrayView<FPBDRigidParticleHandle*>& RigidParticles, const EObjectStateType ObjectState, const bool bUpdateViews, const bool bIsSleeping = false)
	{
		int32 ParticleOffset = 0;
		
		// The particles in [0-ParticleOffset) will be the geometry collection ones one whereas the [ParticleOffset,NumParticles) will be the others
		Private::ReorderParticleView(RigidParticles, [](FPBDRigidParticleHandle* RigidParticle) -> bool 
			{ return RigidParticle && (RigidParticle->CastToGeometryCollection() != nullptr);}, ParticleOffset); 
	
		const int32 NumGeometryCollection = ParticleOffset;
		if (NumGeometryCollection)
		{
			TArrayView<FPBDRigidParticleHandle*> GCParticles(&RigidParticles[0], NumGeometryCollection);
			UpdateClusteredParticlesSOA(GCParticles, ObjectState, *GeometryCollectionParticles, ParticleOffset);
			RemoveGeometryCollectionParticles(GCParticles);
			
			const int32 NumEnabled = GCParticles.Num() - ParticleOffset;
			if (NumEnabled > 0)
			{
				TArrayView<FPBDRigidParticleHandle*> EnabledParticles(&GCParticles[ParticleOffset], NumEnabled);
				InsertGeometryCollectionParticles(EnabledParticles, false, false, ObjectState);	
			}		
		}
	
		const int32 NumClustered = RigidParticles.Num()-NumGeometryCollection;
		if (NumClustered)
		{
			TArrayView<FPBDRigidParticleHandle*> CLParticles(&RigidParticles[NumGeometryCollection], NumClustered);
			UpdateClusteredParticlesSOA(CLParticles, ObjectState, *ClusteredParticles, ParticleOffset);
			RemoveClusteredParticles(CLParticles);
			
			const int32 NumEnabled = CLParticles.Num() - ParticleOffset;
			if (NumEnabled > 0)
			{
				TArrayView<FPBDRigidParticleHandle*> EnabledParticles(&CLParticles[ParticleOffset], NumEnabled);
				InsertClusteredParticles(EnabledParticles, false, ObjectState);	
			}		
		}
		PostMoveParticlesSOA(RigidParticles, bUpdateViews);
	}

	void SetClusteredParticleSOA(FPBDRigidClusteredParticleHandle* ClusteredParticle)
	{
		FPBDRigidParticleHandle* RigidParticle = static_cast<FPBDRigidParticleHandle*>(ClusteredParticle);
		TArrayView<FPBDRigidParticleHandle*> RigidParticles(&RigidParticle, 1);
		
		SetClusteredParticlesSOA(RigidParticles, ClusteredParticle->ObjectState(), true, ClusteredParticle->Sleeping());
	}

	void MarkMovingKinematic(FKinematicGeometryParticleHandle* Particle)
	{
		if (FPBDRigidParticleHandle* Rigid = Particle->CastToRigidParticle())
		{
			AddToList(Rigid, MovingKinematicsMapArray);
			// Remove from TransientDirtyMapArray to be sure not have have particle in both lists
			RemoveFromList(Rigid, TransientDirtyMapArray);

			CheckParticleViewMask(Particle);
		}
	}

	void UpdateAllMovingKinematic(const bool bUpdateViews = true)
	{
		// moving kinematics are going to a 'reset' mode then a 'none' mode
		// 'reset' ones need to stay for another frame 
		// 'none' can be safely removed from the map/array 
		int32 Index = 0;
		while (Index < MovingKinematicsMapArray.GetArray().Num())
		{
			if (MovingKinematicsMapArray.GetArray()[Index]->KinematicTarget().GetMode() == EKinematicTargetMode::None)
			{
				// remove and do not increment Index
				FPBDRigidParticleHandle* Rigid = MovingKinematicsMapArray.GetArray()[Index];
				RemoveFromList(Rigid, MovingKinematicsMapArray);

				// Particle may not be moving, but its velocity was just changed (to zero), so it
				// needs to be in a dirty list. Since it are no longer in MovingKinematicsMapArray
				// we must put it the transient dirty list
				MarkTransientDirtyParticle(Rigid, bUpdateViews);

				CheckParticleViewMask(Rigid);
			}
			else
			{
				++Index;
			}
		}
	}

	void Serialize(FChaosArchive& Ar)
	{
		static const FName SOAsName = "PBDRigidsSOAs";
		FChaosArchiveScopedMemory ScopedMemory(Ar, SOAsName, false);

		ParticleHandles.Serialize(Ar);

		Ar << StaticParticles;
		Ar << StaticDisabledParticles;
		Ar << KinematicParticles;
		Ar << KinematicDisabledParticles;
		Ar << DynamicParticles;
		Ar << DynamicDisabledParticles;
		// TODO: Add an entry in UObject/ExternalPhysicsCustomObjectVersion.h when adding these back in:
		//Ar << ClusteredParticles;
		//Ar << GeometryCollectionParticles;

		Ar.UsingCustomVersion(FExternalPhysicsCustomObjectVersion::GUID);
		if (Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) >= FExternalPhysicsCustomObjectVersion::AddDynamicKinematicSOA)
		{
			Ar << DynamicKinematicParticles;
		}

		{
			//need to assign indices to everything
			auto AssignIdxHelper = [&](const auto& Particles)
			{
				for(uint32 ParticleIdx = 0; ParticleIdx < Particles->Size(); ++ParticleIdx)
				{
					FUniqueIdx Unique = UniqueIndices.GenerateUniqueIdx();
					Particles->UniqueIdx(ParticleIdx) = Unique;
					Particles->GTGeometryParticle(ParticleIdx)->SetUniqueIdx(Unique);
				}
			};

			AssignIdxHelper(StaticParticles);
			AssignIdxHelper(StaticDisabledParticles);
			AssignIdxHelper(KinematicParticles);
			AssignIdxHelper(DynamicParticles);
			AssignIdxHelper(DynamicParticles);
			AssignIdxHelper(DynamicDisabledParticles);
			//AssignIdxHelper(ClusteredParticles);
			//AssignIdxHelper(GeometryCollectionParticles);
		}

		ensure(ClusteredParticles->Size() == 0);	//not supported yet
		//Ar << ClusteredParticles;
		Ar << GeometryCollectionParticles;

		ActiveParticlesMapArray.Serialize(Ar);
		//DynamicClusteredMapArray.Serialize(Ar);

		//todo: update deterministic ID

		// We do not serialize the list masks to simplify maintenance
		if (Ar.IsLoading())
		{
			SetContainerListMasks();
		}

		//if (!GeometryCollectionParticles || !GeometryCollectionParticles->Size())
			UpdateViews();
		//else
		//	UpdateGeometryCollectionViews();
	}


	const TParticleView<FGeometryParticles>& GetNonDisabledView() const {  return NonDisabledView; }

	const TParticleView<FPBDRigidParticles>& GetNonDisabledDynamicView() const { return NonDisabledDynamicView; }

	const TParticleView<FPBDRigidClusteredParticles>& GetNonDisabledClusteredView() const { return NonDisabledClusteredView; }

	const TParticleView<FPBDRigidParticles>& GetActiveParticlesView() const {  return ActiveParticlesView; }
	TParticleView<FPBDRigidParticles>& GetActiveParticlesView() { return ActiveParticlesView; }

	const TArray<FPBDRigidParticleHandle*>& GetActiveParticlesArray() const { return ActiveParticlesMapArray.GetArray(); }
	
	const TParticleView<FPBDRigidParticles>& GetDirtyParticlesView() const { return DirtyParticlesView; }
	TParticleView<FPBDRigidParticles>& GetDirtyParticlesView() { return DirtyParticlesView; }

	const TParticleView<FGeometryParticles>& GetAllParticlesView() const { return AllParticlesView; }


	const TParticleView<FKinematicGeometryParticles>& GetActiveKinematicParticlesView() const { return ActiveKinematicParticlesView; }
	TParticleView<FKinematicGeometryParticles>& GetActiveKinematicParticlesView() { return ActiveKinematicParticlesView; }

	const TParticleView<FPBDRigidParticles>& GetActiveMovingKinematicParticlesView() const { return ActiveMovingKinematicParticlesView; }
	TParticleView<FPBDRigidParticles>& GetActiveMovingKinematicParticlesView() { return ActiveMovingKinematicParticlesView; }

	const TParticleView<FGeometryParticles>& GetActiveStaticParticlesView() const { return ActiveStaticParticlesView; }
	TParticleView<FGeometryParticles>& GetActiveStaticParticlesView() { return ActiveStaticParticlesView; }

	const TParticleView<FPBDRigidParticles>& GetActiveDynamicMovingKinematicParticlesView() const { return ActiveDynamicMovingKinematicParticlesView; }
	TParticleView<FPBDRigidParticles>& GetActiveDynamicMovingKinematicParticlesView() { return ActiveDynamicMovingKinematicParticlesView; }

	const TGeometryParticleHandles<FReal, 3>& GetParticleHandles() const { return ParticleHandles; }
	TGeometryParticleHandles<FReal, 3>& GetParticleHandles() { return ParticleHandles; }

	const FPBDRigidParticles& GetDynamicParticles() const { return *DynamicParticles; }
	FPBDRigidParticles& GetDynamicParticles() { return *DynamicParticles; }

	const FPBDRigidParticles& GetDynamicKinematicParticles() const { return *DynamicKinematicParticles; }
	FPBDRigidParticles& GetDynamicKinematicParticles() { return *DynamicKinematicParticles; }

	// Disabled Dynamic and DynamicKinematic Particles
	const FPBDRigidParticles& GetDynamicDisabledParticles() const { return *DynamicDisabledParticles; }
	FPBDRigidParticles& GetDynamicDisabledParticles() { return *DynamicDisabledParticles; }

	const FGeometryParticles& GetNonDisabledStaticParticles() const { return *StaticParticles; }
	FGeometryParticles& GetNonDisabledStaticParticles() { return *StaticParticles; }

	const TPBDGeometryCollectionParticles<FReal, 3>& GetGeometryCollectionParticles() const { return *GeometryCollectionParticles; }
	TPBDGeometryCollectionParticles<FReal, 3>& GetGeometryCollectionParticles() { return *GeometryCollectionParticles; }

	const TArray<FPBDGeometryCollectionParticleHandle*>& GetSleepingGeometryCollectionArray() const { return SleepingGeometryCollectionArray.GetArray(); }
	const TArray<FPBDGeometryCollectionParticleHandle*>& GetDynamicGeometryCollectionArray() const { return DynamicGeometryCollectionArray.GetArray(); }

	void InsertGeometryCollectionParticles(const TArrayView<FPBDRigidParticleHandle*>& RigidParticles, const bool bIsDisabled, const bool bIsSleeping, const EObjectStateType ObjectState)
	{
		if (!bIsDisabled)
		{
			const Chaos::EObjectStateType State = bIsSleeping ? Chaos::EObjectStateType::Sleeping : ObjectState;
			switch (State)
			{
				case EObjectStateType::Uninitialized:
					ensure(false); // we should probably not be here 
					break;
				case EObjectStateType::Static:
					AddToList(RigidParticles, StaticGeometryCollectionArray);
					break;
				case EObjectStateType::Kinematic:
					AddToList(RigidParticles, KinematicGeometryCollectionArray);
					break;
				case EObjectStateType::Dynamic:
					AddToList(RigidParticles, DynamicGeometryCollectionArray);
					break;
				case EObjectStateType::Sleeping:
					AddToList(RigidParticles, SleepingGeometryCollectionArray);
					break;
				default : 
					break;
			}
		}
	}

	void InsertGeometryCollectionParticle(TPBDGeometryCollectionParticleHandle<FReal, 3>* GCParticle)
	{
		FPBDRigidParticleHandle* RigidParticle = static_cast<FPBDRigidParticleHandle*>(GCParticle);
		InsertGeometryCollectionParticles(MakeArrayView<FPBDRigidParticleHandle*>(&RigidParticle, 1), 
			GCParticle->Disabled(), GCParticle->Sleeping(), GCParticle->ObjectState());
	}

	void RemoveGeometryCollectionParticles(const TArrayView<FPBDRigidParticleHandle*>& GCParticles)
	{
		RemoveFromList(GCParticles, StaticGeometryCollectionArray);
		RemoveFromList(GCParticles, KinematicGeometryCollectionArray);
		RemoveFromList(GCParticles, SleepingGeometryCollectionArray);
		RemoveFromList(GCParticles, DynamicGeometryCollectionArray);
	}

	void RemoveGeometryCollectionParticle(TPBDGeometryCollectionParticleHandle<FReal, 3>* GCParticle)
	{
		FPBDRigidParticleHandle* RigidParticle = static_cast<FPBDRigidParticleHandle*>(GCParticle);
		RemoveGeometryCollectionParticles(MakeArrayView<FPBDRigidParticleHandle*>(&RigidParticle, 1));
	}

	const auto& GetClusteredParticles() const { return *ClusteredParticles; }
	auto& GetClusteredParticles() { return *ClusteredParticles; }

	auto& GetUniqueIndices() { return UniqueIndices; }

	// Check that the particles in each of the particle lists have their ListMask sett appropriately
	CHAOS_API void CheckListMasks();
	
	// Check that no particles are in multiple lists that contribute to a single view
	CHAOS_API void CheckViewMasks();

private:

	//
	// List Tracking System: 
	// every particle stores the set of SOAs/ParticleMapArrays/ParticleArrays 
	// that the particle is in. This information is used to check for invalid 
	// combinations (some list memberships are mutually exclusive) and to 
	// reduce the cost of checking whether a particle is in a particular list.
	//

	// Initialize the list mask for all particle containers and lists
	CHAOS_API void SetContainerListMasks();
	
	// Add a set of particles to the specified SOA/List and update their ListMasks
	template<typename TListType, typename TParticleHandleType>
	void AddToList(const TArrayView<TParticleHandleType*> Particles, TListType& List)
	{
		check(List.GetContainerListMask() != EGeometryParticleListMask::None);

		for (TParticleHandleType* Particle : Particles)
		{
			Particle->AddToLists(List.GetContainerListMask());
		}
		List.Insert(Particles);
	}

	// Add a particle to the specified SOA/List and update its ListMask
	template<typename TListType, typename TParticleHandleType>
	void AddToList(TParticleHandleType* Particle, TListType& List)
	{
		AddToList(MakeArrayView<TParticleHandleType*>(&Particle, 1), List);
	}
	
	// Remove a set of particles from the specified SOA/List and update their ListMask
	template<typename TListType, typename TParticleHandleType>
	void RemoveFromList(const TArrayView<TParticleHandleType*>& Particles, TListType& List)
	{
		check(List.GetContainerListMask() != EGeometryParticleListMask::None);

		for (TParticleHandleType* Particle : Particles)
		{
			Particle->RemoveFromLists(List.GetContainerListMask());
		}
		List.Remove(Particles);
	}

	// Remove a particle from the specified SOA/List and update its ListMask
	template<typename TListType, typename TParticleHandleType>
	void RemoveFromList(TParticleHandleType* Particle, TListType& List)
	{
		check(List.GetContainerListMask() != EGeometryParticleListMask::None);
		Particle->RemoveFromLists(List.GetContainerListMask());
		List.Remove(Particle);
	}

	// Check that the particles in an SOA have the appropriate bit set in their ListMask
	template<typename TSOAType>
	void CheckSOAMasks(TSOAType* SOA)
	{
		TArray<TSOAView<FGeometryParticles>> SOAViewArray = { SOA };
		auto View = MakeConstParticleView(MoveTemp(SOAViewArray));
		for (const auto& Particle : View)
		{
			ensureAlwaysMsgf(Particle.IsInAnyList(SOA->GetContainerListMask()), TEXT("Particle %s ListMask missing expected SOA bit 0x%x"), *GetParticleDebugName(Particle), EnumToUnderlyingType(SOA->GetContainerListMask()));
		}
	}

	// Check that the particles in a List have the appropriate bit set in their ListMask
	template<typename TListType>
	void CheckListMasks(TListType& List)
	{
		TSOAView<FGeometryParticles> SOAView(&List.GetArray());
		TArray<TSOAView<FGeometryParticles>> SOAViewArray = { SOAView };
		auto View = MakeConstParticleView(MoveTemp(SOAViewArray));
		for (const auto& Particle : View)
		{
			ensureAlwaysMsgf(Particle.IsInAnyList(List.GetContainerListMask()), TEXT("Particle %s ListMask missing expected List bit 0x%x"), *GetParticleDebugName(Particle), EnumToUnderlyingType(List.GetContainerListMask()));
		}
	}

	// Make sure the particle is not in a set of lists that would cause it to appear in any single view multiple times
	void CheckParticleViewMask(const FGeometryParticleHandle* Particle) const
	{
		// This is called after every change to a particle's SOA/List for validation. 
		// It should be cheap when the cvar is disabled but we fully remove it in shipping anyway
#if !UE_BUILD_SHIPPING
		if (CVars::bChaosSolverCheckParticleViews)
		{
			CheckParticleViewMaskImpl(Particle);
		}
#endif
	}

	// Make sure the particle is not in a set of lists that would cause it to appear in any single view multiple times
	CHAOS_API void CheckParticleViewMaskImpl(const FGeometryParticleHandle* Particle) const;


	template <typename TParticleHandleType, typename TParticles>
	TArray<TParticleHandleType*> CreateParticlesHelper(int32 NumParticles, const FUniqueIdx* ExistingIndices,  TUniquePtr<TParticles>& Particles, const FGeometryParticleParameters& Params)
	{
		const int32 ParticlesStartIdx = Particles->Size();
		Particles->AddParticles(NumParticles);
		TArray<TParticleHandleType*> ReturnHandles;
		ReturnHandles.AddUninitialized(NumParticles);

		const int32 HandlesStartIdx = ParticleHandles.Size();
		ParticleHandles.AddHandles(NumParticles);

		for (int32 Count = 0; Count < NumParticles; ++Count)
		{
			const int32 ParticleIdx = Count + ParticlesStartIdx;
			const int32 HandleIdx = Count + HandlesStartIdx;

			TUniquePtr<TParticleHandleType> NewParticleHandle = TParticleHandleType::CreateParticleHandle(MakeSerializable(Particles), ParticleIdx, HandleIdx);
			NewParticleHandle->SetParticleID(FParticleID{ INDEX_NONE, BiggestParticleID++ });
			NewParticleHandle->AddToLists(Particles->GetContainerListMask());
			ReturnHandles[Count] = NewParticleHandle.Get();
			//If unique indices are null it means there is no GT particle that already registered an ID, so create one
			if(ExistingIndices)
			{
				ReturnHandles[Count]->SetUniqueIdx(ExistingIndices[Count]);
			}
			else
			{
				ReturnHandles[Count]->SetUniqueIdx(UniqueIndices.GenerateUniqueIdx());
			}
			ParticleHandles.Handle(HandleIdx) = MoveTemp(NewParticleHandle);
			Particles->HasCollision(ParticleIdx) = true;	//todo: find a better place for this
		}

		return ReturnHandles;
	}
	
	void AddToActiveArray(const TArrayView<FPBDRigidParticleHandle*>& Particles)
	{
		AddToList(Particles, ActiveParticlesMapArray);

		if(bResimulating)
		{
			AddToList(Particles, ResimActiveParticlesMapArray);
		}
		
		//dirty contains Active so make sure no duplicates
		for(FPBDRigidParticleHandle* Particle : Particles)
		{
			RemoveFromList(Particle, TransientDirtyMapArray);
		}
	}

	void AddToActiveArray(FPBDRigidParticleHandle* Particle)
	{
		TArrayView<FPBDRigidParticleHandle*> RigidParticles(&Particle, 1);
		AddToActiveArray(RigidParticles);
	}

	void RemoveFromActiveArray(TArrayView<FPBDRigidParticleHandle*>& RigidParticles, bool bStillDirty)
	{
		RemoveFromList(RigidParticles, ActiveParticlesMapArray);

		if (bResimulating)
		{
			RemoveFromList(RigidParticles, ResimActiveParticlesMapArray);
		}

		if (bStillDirty)
		{
			int32 ParticleOffset = 0;
			
			// The particles in [0-ParticleOffset) will be the enabled ones switching to the new state
			Private::ReorderParticleView(RigidParticles, [this](FPBDRigidParticleHandle* RigidParticle) -> bool 
				{ return !MovingKinematicsMapArray.Contains(RigidParticle);},ParticleOffset);	
		
			const int32 NumTransients = ParticleOffset;
			if (NumTransients > 0)
			{
				// Mark transient particles dirty
				TArrayView<FPBDRigidParticleHandle*> TransientParticles(&RigidParticles[0], NumTransients);
				AddToList(TransientParticles, TransientDirtyMapArray);
			}
		}
		else
		{
			//might have already been removed from active from a previous call
			//but now removing and don't want it dirty either
			RemoveFromList(RigidParticles, TransientDirtyMapArray);
		}
	}

	void RemoveFromActiveArray(FPBDRigidParticleHandle* Particle, bool bStillDirty)
	{
		TArrayView<FPBDRigidParticleHandle*> RigidParticles(&Particle, 1);
		RemoveFromActiveArray(RigidParticles, bStillDirty);
	}
	
	//should be called whenever particles are added / removed / reordered
	CHAOS_API void UpdateViews();

	void InsertClusteredParticle(FPBDRigidClusteredParticleHandle* ClusteredParticle)
	{
		FPBDRigidParticleHandle* RigidParticle = static_cast<FPBDRigidParticleHandle*>(ClusteredParticle);
		TArrayView<FPBDRigidParticleHandle*> ClusteredParticlesView(&RigidParticle, 1);
		
		InsertClusteredParticles(ClusteredParticlesView, ClusteredParticle->Disabled(), ClusteredParticle->ObjectState());
	}

	void InsertClusteredParticles(const TArray<FPBDRigidClusteredParticleHandle*>& Particles)
	{
		// We still need to loop over each particle since each one of them could have different object state 
		// and could potentially be disabled
		for (FPBDRigidClusteredParticleHandle* ClusteredParticle : Particles)
		{
			InsertClusteredParticle(ClusteredParticle);
		}
	}

	void InsertClusteredParticles(const TArrayView<FPBDRigidParticleHandle*>& RigidParticles, const bool bIsDisabled, const EObjectStateType ObjectState)
	{
		if (!bIsDisabled)
		{
			switch (ObjectState)
			{
				case EObjectStateType::Uninitialized:
					ensure(false); // we should probably not be here 
					break;
				case EObjectStateType::Static:
					AddToList(RigidParticles, StaticClusteredMapArray);
					break;
				case EObjectStateType::Kinematic:
					AddToList(RigidParticles, KinematicClusteredMapArray);
					break;
				case EObjectStateType::Dynamic:
				case EObjectStateType::Sleeping:
					AddToList(RigidParticles, DynamicClusteredMapArray);
					break;
				default : 
					break;
			}
		}
	}

	void RemoveClusteredParticle(FPBDRigidClusteredParticleHandle* ClusteredParticle)
	{
		FPBDRigidParticleHandle* RigidParticle = static_cast<FPBDRigidParticleHandle*>(ClusteredParticle);
		TArrayView<FPBDRigidParticleHandle*> ClusteredParticlesView(&RigidParticle, 1);
		
		RemoveClusteredParticles(ClusteredParticlesView);
	}

	void RemoveClusteredParticles(const TArrayView<FPBDRigidParticleHandle*>& RigidParticles)
	{
		RemoveFromList(RigidParticles, StaticClusteredMapArray);
		RemoveFromList(RigidParticles, KinematicClusteredMapArray);
		RemoveFromList(RigidParticles, DynamicClusteredMapArray);
	}

	//Organized by SOA type
	TUniquePtr<FGeometryParticles> StaticParticles;
	TUniquePtr<FGeometryParticles> StaticDisabledParticles;

	TUniquePtr<FKinematicGeometryParticles> KinematicParticles;
	TUniquePtr<FKinematicGeometryParticles> KinematicDisabledParticles;

	TUniquePtr<FPBDRigidParticles> DynamicParticles;
	TUniquePtr<FPBDRigidParticles> DynamicKinematicParticles;
	TUniquePtr<FPBDRigidParticles> DynamicDisabledParticles;

	TUniquePtr<FPBDRigidClusteredParticles> ClusteredParticles;

	TUniquePtr<TPBDGeometryCollectionParticles<FReal, 3>> GeometryCollectionParticles;

	//
	// NOTE: The member here are enumerated in EGeometryParticleListMask which must
	// be kept up to date with changes here. Also see SetContainerListMasks() and CheckListMasks()
	//

	// Geometry collection particle state is controlled via their disabled state and assigned 
	// EObjectStateType, and are shuffled into these corresponding arrays in UpdateGeometryCollectionViews().
	TParticleMapArray<FPBDGeometryCollectionParticleHandle> StaticGeometryCollectionArray;
	TParticleMapArray<FPBDGeometryCollectionParticleHandle> KinematicGeometryCollectionArray;
	TParticleMapArray<FPBDGeometryCollectionParticleHandle> SleepingGeometryCollectionArray;
	TParticleMapArray<FPBDGeometryCollectionParticleHandle> DynamicGeometryCollectionArray;

	// Utility structures for maintaining an Active particles view
	TParticleMapArray<FPBDRigidParticleHandle> ActiveParticlesMapArray;
	TParticleMapArray<FPBDRigidParticleHandle> TransientDirtyMapArray;
	
	// Keep track of kinematic that have their kinematic target set for this current frame
	TParticleMapArray<FPBDRigidParticleHandle> MovingKinematicsMapArray;

	// Structures for maintaining a subset view during a resim (TParticleMapArray used when we need to dynamically add/remove during resim)
	TParticleMapArray<FPBDRigidParticleHandle> ResimActiveParticlesMapArray;
	TParticleMapArray<FPBDRigidParticleHandle> ResimDynamicParticles;
	TParticleMapArray<FPBDRigidParticleHandle> ResimDynamicKinematicParticles;
	TParticleArray<FGeometryParticleHandle> ResimStaticParticles;
	TParticleArray<FKinematicGeometryParticleHandle> ResimKinematicParticles;
	bool bResimulating = false;

	// NonDisabled clustered particle arrays
	TParticleMapArray<FPBDRigidClusteredParticleHandle> StaticClusteredMapArray;
	TParticleMapArray<FPBDRigidClusteredParticleHandle> KinematicClusteredMapArray;
	TParticleMapArray<FPBDRigidClusteredParticleHandle> DynamicClusteredMapArray;

	// Particle Views
	TParticleView<FGeometryParticles> NonDisabledView;								//all particles that are not disabled
	TParticleView<FPBDRigidParticles> NonDisabledDynamicView;						//all dynamic particles that are not disabled
	TParticleView<FPBDRigidClusteredParticles> NonDisabledClusteredView;			//all clustered particles that are not disabled
	TParticleView<FPBDRigidParticles> ActiveParticlesView;							//all particles that are active
	TParticleView<FPBDRigidParticles> DirtyParticlesView;							//all particles that are active + any that were put to sleep this frame
	TParticleView<FGeometryParticles> AllParticlesView;								//all particles
	TParticleView<FKinematicGeometryParticles> ActiveKinematicParticlesView;		//all kinematic particles that are not disabled
	TParticleView<FPBDRigidParticles> ActiveMovingKinematicParticlesView;			//all moving kinematic particles that are not disabled
	TParticleView<FPBDRigidParticles> ActiveDynamicMovingKinematicParticlesView;	//all moving kinematic particles that are not disabled + all dynamic particles
	TParticleView<FGeometryParticles> ActiveStaticParticlesView;					//all static particles that are not disabled
	TParticleView<TPBDGeometryCollectionParticles<FReal, 3>> ActiveGeometryCollectionParticlesView; // all geom collection particles that are not disabled

	// Auxiliary data synced with particle handles
	TGeometryParticleHandles<FReal, 3> ParticleHandles;

	IParticleUniqueIndices& UniqueIndices;

#if CHAOS_DETERMINISTIC
	int32 BiggestParticleID;
#endif
};

template <typename T, int d>
using TPBDRigidsSOAs UE_DEPRECATED(4.27, "Deprecated. this class is to be deleted, use FPBDRigidsSOAs instead") = FPBDRigidsSOAs;

}
