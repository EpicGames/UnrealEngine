// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if !COMPILE_WITHOUT_UNREAL_SUPPORT
#include "Chaos/Core.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "Chaos/PBDSoftsEvolutionFwd.h"
#include "Chaos/PBDSoftsSolverParticles.h"
#include "Chaos/SoftsSolverParticlesRange.h"
#include "Containers/Set.h"

namespace Chaos::Softs
{
	class FPBDSelfCollisionSphereConstraintsBase
	{
	public:
		CHAOS_API FPBDSelfCollisionSphereConstraintsBase(
			const int32 InOffset,
			const int32 InNumParticles,
			const TSet<int32>* InVertexSetNoOffset,
			const TArray<FSolverVec3>* InReferencePositions,
			const FSolverReal InRadius,
			const FSolverReal InStiffness);

		virtual ~FPBDSelfCollisionSphereConstraintsBase() = default;

		template<typename SolverParticlesOrRange>
		CHAOS_API void Init(const SolverParticlesOrRange& Particles);

		template<typename SolverParticlesOrRange>
		CHAOS_API void Apply(SolverParticlesOrRange& InParticles, const FSolverReal Dt) const;

		const TArray<TVec2<int32>>& GetConstraints() const { return Constraints; }
		const TSet<int32>* GetVertexSet() const { return VertexSetNoOffset; }
		FSolverReal GetRadius() const { return Radius; }

	protected:
		TArray<TVec2<int32>> Constraints;
		FSolverReal Radius;
		FSolverReal Stiffness;
		const TSet<int32>* VertexSetNoOffset;

	private:
		int32 Offset;
		int32 NumParticles;
		const TArray<FSolverVec3>* ReferencePositions;
	};

	class FPBDSelfCollisionSphereConstraints : public FPBDSelfCollisionSphereConstraintsBase
	{
		typedef FPBDSelfCollisionSphereConstraintsBase Base;

	public:
		static bool IsEnabled(const FCollectionPropertyConstFacade& PropertyCollection)
		{
			return IsSelfCollisionSphereStiffnessEnabled(PropertyCollection, false);
		}

		FPBDSelfCollisionSphereConstraints(
			const int32 InOffset,
			const int32 InNumParticles,
			const TMap<FString, const TSet<int32>*>& VertexSets,
			const FCollectionPropertyConstFacade& PropertyCollection)
			: Base(InOffset,
				InNumParticles,
				VertexSets.FindRef(GetSelfCollisionSphereSetNameString(PropertyCollection, SelfCollisionSphereSetNameName.ToString()), nullptr),
				nullptr, // No reference positions--using vertex sets instead
				(FSolverReal)FMath::Max(GetSelfCollisionSphereRadius(PropertyCollection, 0.5f), 0.f),
				(FSolverReal)FMath::Clamp(GetSelfCollisionSphereStiffness(PropertyCollection, 0.5f), 0.f, 1.f)),
			SelfCollisionSphereRadiusIndex(PropertyCollection),
			SelfCollisionSphereStiffnessIndex(PropertyCollection)
		{}

		CHAOS_API void SetProperties(
			const FCollectionPropertyConstFacade& PropertyCollection,
			const TMap<FString, const TSet<int32>*>& VertexSets);

		UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(SelfCollisionSphereSetName, bool);  // Selection set name string property, the bool value is not actually used
		UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(SelfCollisionSphereRadius, float);
		UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(SelfCollisionSphereStiffness, float);

	private:
		using Base::Radius;
		using Base::Stiffness;
		using Base::VertexSetNoOffset;

		UE_CHAOS_DECLARE_INDEXED_PROPERTYCOLLECTION_NAME(SelfCollisionSphereRadius, float);
		UE_CHAOS_DECLARE_INDEXED_PROPERTYCOLLECTION_NAME(SelfCollisionSphereStiffness, float);
	};
}
#endif
