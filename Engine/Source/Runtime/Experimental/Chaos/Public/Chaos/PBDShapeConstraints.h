// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// HEADER_UNIT_SKIP - Internal

#include "Chaos/PBDShapeConstraintsBase.h"

namespace Chaos::Softs
{

class FPBDShapeConstraints : public FPBDShapeConstraintsBase
{
	typedef FPBDShapeConstraintsBase Base;
	using Base::ParticleOffset;
	using Base::TargetPositions;

public:

	FPBDShapeConstraints(
		int32 InParticleOffset,
		int32 InParticleCount,
		const TArray<FSolverVec3>& StartPositions,
		const TArray<FSolverVec3>& TargetPositions,
		const FSolverReal Stiffness = (FSolverReal)1.
	)
		: Base(InParticleOffset, InParticleCount, StartPositions, TargetPositions, Stiffness)
	{
	}
	virtual ~FPBDShapeConstraints() override = default;

	void Apply(FSolverParticles& InParticles, const FSolverReal Dt, const int32 Index) const
	{
		if (InParticles.InvM(Index) > (FSolverReal)0.)
		{
			InParticles.P(Index) -= InParticles.InvM(Index) * Base::GetDelta(InParticles, Index);
		}
	}

	void Apply(FSolverParticles& InParticles, const FSolverReal Dt) const
	{
		for (int32 Index = ParticleOffset; Index < ParticleOffset + TargetPositions.Num(); ++Index)
		{
			Apply(InParticles, Dt, Index);
		}
	}
};

}  // End namespace Chaos::Softs
