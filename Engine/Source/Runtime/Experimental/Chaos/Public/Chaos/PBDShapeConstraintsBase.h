// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/PBDSoftsEvolutionFwd.h"
#include "Chaos/Array.h"
#include "Chaos/PBDParticles.h"
#include "Chaos/PBDSoftsSolverParticles.h"

#include <functional>

namespace Chaos::Softs
{

class FPBDShapeConstraintsBase
{
public:
	FPBDShapeConstraintsBase(
		int32 InParticleOffset,
		int32 InParticleCount,
		const TArray<FSolverVec3>& StartPositions,
		const TArray<FSolverVec3>& InTargetPositions,
		const FSolverReal InStiffness
	)
		: TargetPositions(InTargetPositions)
		, ParticleOffset(InParticleOffset)
		, Stiffness(InStiffness)
	{
		const int32 NumConstraints = InParticleCount;
		Dists.SetNumUninitialized(InParticleCount);
		for (int32 Index = 0; Index < InParticleCount; ++Index)
		{
			const int32 ParticleIndex = ParticleOffset + Index;
			const FSolverVec3& P1 = StartPositions[ParticleIndex];
			const FSolverVec3& P2 = TargetPositions[ParticleIndex];
			Dists[Index] = (P1 - P2).Size();
		}
	}
	virtual ~FPBDShapeConstraintsBase() = default;

	FSolverVec3 GetDelta(const FSolverParticles& InParticles, const int32 Index) const
	{
		checkSlow(Index >= ParticleOffset && Index < ParticleOffset + Dists.Num())
		if (InParticles.InvM(Index) == (FSolverReal)0.)
		{
			return FSolverVec3(0.);
		}
		const FSolverVec3& P1 = InParticles.P(Index);
		const FSolverVec3& P2 = TargetPositions[Index];
		const FSolverVec3 Difference = P1 - P2;
		const FSolverReal Distance = Difference.Size();
		const FSolverVec3 Direction = Difference / Distance;
		const FSolverVec3 Delta = (Distance - Dists[Index - ParticleOffset]) * Direction;
		return Stiffness * Delta / InParticles.InvM(Index);
	}

protected:
	const TArray<FSolverVec3>& TargetPositions;
	const int32 ParticleOffset;

private:
	TArray<FSolverReal> Dists;
	FSolverReal Stiffness;
};

}  // End namespace Chaos::Softs
