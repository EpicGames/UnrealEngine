// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/CollectionPropertyFacade.h"
#include "Chaos/Core.h"
#include "Chaos/PBDKinematicTriangleMeshCollisions.h"
#include "Chaos/PBDSoftsEvolutionFwd.h"
#include "Chaos/PBDSoftsSolverParticles.h"
#include "Chaos/PBDFlatWeightMap.h"
#include "Chaos/PBDWeightMap.h"
#include "Chaos/PBDPlanarConstraints.h"
#include "Chaos/SoftsSolverCollisionParticles.h"
#include "Chaos/SoftsSolverCollisionParticlesRange.h"
#include "Chaos/SoftsSolverParticlesRange.h"

#if !defined(CHAOS_PER_PARTICLE_COLLISION_ISPC_ENABLED_DEFAULT)
#define CHAOS_PER_PARTICLE_COLLISION_ISPC_ENABLED_DEFAULT 1
#endif

// Support run-time toggling on supported platforms in non-shipping configurations
#if !INTEL_ISPC || UE_BUILD_SHIPPING
const bool bChaos_SoftBodyCollision_ISPC_Enabled = INTEL_ISPC && CHAOS_PER_PARTICLE_COLLISION_ISPC_ENABLED_DEFAULT;
#else
extern CHAOS_API bool bChaos_SoftBodyCollision_ISPC_Enabled;
#endif

namespace Chaos::Softs
{
	struct FPBDComplexColliderBoneData
	{
		// Use this array to go from BoneData indices to the X, R, V, etc. arrays below
		TConstArrayView<int32> MappedBoneIndices;

		TConstArrayView<FSolverVec3> X;
		TConstArrayView<FSolverVec3> V;
		TConstArrayView<FSolverRotation3> R;
		TConstArrayView<FSolverVec3> W;
	};

class FPBDSoftBodyCollisionConstraintBase
{
public:
	static constexpr bool bDefaultUsePlanarConstraintForSimpleColliders = false;
	static constexpr bool bDefaultUsePlanarConstraintForComplexColliders = true;

	FPBDSoftBodyCollisionConstraintBase(
		const TArray<FSolverRigidTransform3>& InLastCollisionTransforms,
		FSolverReal InCollisionThickness,
		FSolverReal InFrictionCoefficient,
		bool bInUseCCD,
		FSolverReal InProximityStiffness,
		TArray<bool>* InCollisionParticleCollided = nullptr,
		TArray<FSolverVec3>* InContacts = nullptr,
		TArray<FSolverVec3>* InNormals = nullptr,
		TArray<FSolverReal>* InPhis = nullptr,
		const FSolverVec2 InSoftBodyCollisionThickness = FSolverVec2::ZeroVector,
		bool bInEnableSimpleColliders = true,
		bool bInEnableComplexColliders = true,
		bool bInUsePlanarConstraintForSimpleColliders = bDefaultUsePlanarConstraintForSimpleColliders,
		bool bInUsePlanarConstraintForComplexColliders = bDefaultUsePlanarConstraintForComplexColliders,
		const TMap<FParticleRangeIndex, FPBDComplexColliderBoneData>& InComplexBoneData = TMap<FParticleRangeIndex, FPBDComplexColliderBoneData>()
		)
		: LastCollisionTransforms(InLastCollisionTransforms)
		, InnerCollisionThickness(FSolverVec2::ZeroVector)
		, CollisionThickness(InCollisionThickness)
		, SoftBodyCollisionThicknessWeighted(InSoftBodyCollisionThickness, TConstArrayView<FRealSingle>(), 0)
		, FrictionCoefficient(InFrictionCoefficient)
		, bUseCCD(bInUseCCD)
		, ProximityStiffness(InProximityStiffness)
		, bEnableSimpleColliders(bInEnableSimpleColliders)
		, bEnableComplexColliders(bInEnableComplexColliders)
		, bUsePlanarConstraintForSimpleColliders(bInUsePlanarConstraintForSimpleColliders)
		, bUsePlanarConstraintForComplexColliders(bInUsePlanarConstraintForComplexColliders)
		, ComplexBoneData(InComplexBoneData)
		, CollisionParticleCollided(InCollisionParticleCollided)
		, Contacts(InContacts)
		, Normals(InNormals)
		, Phis(InPhis)
		, PlanarConstraint(FrictionCoefficient)
	{}

	UE_DEPRECATED(5.8, "Use the overload taking FSolverVec2 for InSoftBodyCollisionThickness")
	FPBDSoftBodyCollisionConstraintBase(
		const TArray<FSolverRigidTransform3>& InLastCollisionTransforms,
		FSolverReal InCollisionThickness,
		FSolverReal InFrictionCoefficient,
		bool bInUseCCD,
		FSolverReal InProximityStiffness,
		TArray<bool>* InCollisionParticleCollided,
		TArray<FSolverVec3>* InContacts,
		TArray<FSolverVec3>* InNormals,
		TArray<FSolverReal>* InPhis,
		const FSolverReal InSoftBodyCollisionThickness,
		bool bInEnableSimpleColliders = true,
		bool bInEnableComplexColliders = true,
		bool bInUsePlanarConstraintForSimpleColliders = bDefaultUsePlanarConstraintForSimpleColliders,
		bool bInUsePlanarConstraintForComplexColliders = bDefaultUsePlanarConstraintForComplexColliders,
		const TMap<FParticleRangeIndex, FPBDComplexColliderBoneData>& InComplexBoneData = TMap<FParticleRangeIndex, FPBDComplexColliderBoneData>()
		)
		: FPBDSoftBodyCollisionConstraintBase(InLastCollisionTransforms, InCollisionThickness,
			InFrictionCoefficient, bInUseCCD, InProximityStiffness,
			InCollisionParticleCollided, InContacts, InNormals, InPhis,
			FSolverVec2(InSoftBodyCollisionThickness, InSoftBodyCollisionThickness),
			bInEnableSimpleColliders, bInEnableComplexColliders,
			bInUsePlanarConstraintForSimpleColliders, bInUsePlanarConstraintForComplexColliders,
			InComplexBoneData)
	{}

	void SetWriteDebugContacts(bool bWrite) { bWriteDebugContacts = bWrite; }

	// Follow planar constraint settings and either generate or apply planar constraints if requested.
	CHAOS_API void ApplyWithPlanarConstraints(FSolverParticlesRange& Particles, const FSolverReal Dt, const TArray<FSolverCollisionParticlesRange>& CollisionParticles, bool bGeneratePlanarConstraints);

	// Ignore planar constraint settings and do a full collision detection and apply impulses
	CHAOS_API void Apply(FSolverParticlesRange& Particles, const FSolverReal Dt, const TArray<FSolverCollisionParticlesRange>& CollisionParticles);
	
	CHAOS_API void UpdateLinearSystem(const FSolverParticlesRange& Particles, const FSolverReal Dt, const TArray<FSolverCollisionParticlesRange>& CollisionParticles, FEvolutionLinearSystem& LinearSystem) const;

	void OnCollisionRangeRemoved(int32 CollisionRangeId)
	{
		for (TMap<FParticleRangeIndex, FPBDComplexColliderBoneData>::TIterator Iter = ComplexBoneData.CreateIterator(); Iter; ++Iter)
		{
			if (Iter.Key().RangeId == CollisionRangeId)
			{
				Iter.RemoveCurrent();
			}
		}
	}

	FSolverReal GetParticleClothThickness(int32 ParticleIndex) const
	{
		return SoftBodyCollisionThicknessWeighted.GetValue(ParticleIndex);
	}

	FSolverReal GetParticleInnerThickness(int32 ParticleIndex) const
	{
		return InnerCollisionThickness.GetValue(ParticleIndex);
	}

private:

	template<bool bLockAndWriteContacts, bool bWithFriction, bool bGeneratePlanarConstraints>
	void ApplySimpleInternal(FSolverParticlesRange& Particles, const FSolverReal Dt, const TArray<FSolverCollisionParticlesRange>& CollisionParticles);
	template<bool bLockAndWriteContacts, bool bWithFriction, bool bGeneratePlanarConstraints>
	void ApplyComplexInternal(FSolverParticlesRange& Particles, const FSolverReal Dt, const TArray<FSolverCollisionParticlesRange>& CollisionParticles);
	template<bool bLockAndWriteContacts, bool bWithFriction>
	void ApplyInternalCCD(FSolverParticlesRange& Particles, const FSolverReal Dt, const TArray<FSolverCollisionParticlesRange>& CollisionParticles) const;
#if INTEL_ISPC
	void ApplySimpleInternalISPC(FSolverParticlesRange& Particles, const FSolverReal Dt, const TArray<FSolverCollisionParticlesRange>& CollisionParticles, bool bGeneratePlanarConstraints);
	void ApplyComplexInternalISPC(FSolverParticlesRange& Particles, const FSolverReal Dt, const TArray<FSolverCollisionParticlesRange>& CollisionParticles, bool bGeneratePlanarConstraints);
#endif
	void InitPlanarConstraints(const FSolverParticlesRange& Particles, bool bWithFriction);
	void FinalizePlanarConstraints(const FSolverParticlesRange& Particles);
	void ApplyPlanarConstraints(FSolverParticlesRange& Particles, const FSolverReal Dt);

protected:
	const TArray<FSolverRigidTransform3>& LastCollisionTransforms; // Used by CCD
	FPBDWeightMap InnerCollisionThickness;
	FSolverReal CollisionThickness;
	FPBDFlatWeightMap SoftBodyCollisionThicknessWeighted;
	UE_DEPRECATED(5.8, "Use SoftBodyCollisionThicknessWeighted instead")
	FSolverReal SoftBodyCollisionThickness = 0.f;
	FSolverReal FrictionCoefficient;
	bool bUseCCD;
	FSolverReal ProximityStiffness; // Used by force-based solver
	bool bEnableSimpleColliders = true;
	bool bEnableComplexColliders = true;
	bool bUsePlanarConstraintForSimpleColliders = bDefaultUsePlanarConstraintForSimpleColliders;
	bool bUsePlanarConstraintForComplexColliders = bDefaultUsePlanarConstraintForComplexColliders;

	TMap<FParticleRangeIndex, FPBDComplexColliderBoneData> ComplexBoneData; // Used to determine Velocity Bones for Skinned LevelSets.

	/**  Used for writing debug contacts */
	bool bWriteDebugContacts = false;
	TArray<bool>* const CollisionParticleCollided; // Per collision particle
	// List of contact data
	TArray<FSolverVec3>* const Contacts;
	TArray<FSolverVec3>* const Normals;
	TArray<FSolverReal>* const Phis;
	mutable FCriticalSection DebugMutex;

	struct FPBDSoftBodyCollisionPlanarConstraint : public FPBDPlanarConstraints
	{
		FPBDSoftBodyCollisionPlanarConstraint() = default;
		~FPBDSoftBodyCollisionPlanarConstraint() = default;
		FPBDSoftBodyCollisionPlanarConstraint(const FSolverReal InFriction)
			:FPBDPlanarConstraints(InFriction)
		{}

		TArray<int32>& GetUniqueConstraintIndices() { return UniqueConstraintIndices; }
		TArray<FSolverVec3>& GetTargetPositions() { return TargetPositions; }
		TArray<FSolverVec3>& GetTargetNormals() { return TargetNormals; }
		TArray<FSolverVec3>& GetTargetVelocities() { return TargetVelocities; }
	} PlanarConstraint;

private:

	// These arrays are used to do batch PhiAndNormal queries. They live here so they don't need to be reallocated each frame.
	TArray<bool> HasPlanarData;
	TArray<FSolverVec3> PlanarDataPositions;
	TArray<FSolverVec3> PlanarDataNormals;
	TArray<FSolverVec3> PlanarDataVelocities;

	TArray<FSolverReal> BatchPhis;
	TArray<FSolverVec3> BatchNormals;
	TArray<int32> BatchVelocityBones;
};


class FPBDSoftBodyCollisionConstraint : public FPBDSoftBodyCollisionConstraintBase
{
	typedef FPBDSoftBodyCollisionConstraintBase Base;

public:
	static constexpr FSolverReal DefaultCollisionThickness = (FSolverReal)1.;
	static constexpr FSolverReal DefaultSoftBodyCollisionThickness = (FSolverReal)0.;
	static constexpr FSolverReal DefaultFrictionCoefficient = (FSolverReal)0.8;
	static constexpr FSolverReal DefaultProximityStiffness = (FSolverReal)100.;

	CHAOS_API FPBDSoftBodyCollisionConstraint(
		const TArray<FSolverRigidTransform3>& InLastCollisionTransforms,
		const FCollectionPropertyConstFacade& PropertyCollection,
		FSolverReal InMeshScale,
		TArray<bool>* InCollisionParticleCollided = nullptr,
		TArray<FSolverVec3>* InContacts = nullptr,
		TArray<FSolverVec3>* InNormals = nullptr,
		TArray<FSolverReal>* InPhis = nullptr,
		const TMap<FParticleRangeIndex, FPBDComplexColliderBoneData>& InComplexBoneData = TMap<FParticleRangeIndex, FPBDComplexColliderBoneData>(),
		const TMap<FString, TConstArrayView<float>>& WeightMaps = TMap<FString, TConstArrayView<float>>());

	CHAOS_API void SetProperties(const FCollectionPropertyConstFacade& PropertyCollection,
		const TMap<FString, TConstArrayView<float>>& WeightMaps);

	UE_DEPRECATED(5.8, "Use the overload that accepts WeightMaps")
	void SetProperties(const FCollectionPropertyConstFacade& PropertyCollection)
	{
		SetProperties(PropertyCollection, TMap<FString, TConstArrayView<float>>());
	}

private:
	const FSolverReal MeshScale;

	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(CollisionThickness, float);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(ClothCollisionThickness, float);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(FrictionCoefficient, float);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(UseCCD, bool);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(ProximityStiffness, float);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(EnableSimpleColliders, bool);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(UsePlanarConstraintForSimpleColliders, bool);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(EnableComplexColliders, bool);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(UsePlanarConstraintForComplexColliders, bool);

	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(InnerCollisionThickness, float);
};

}  // End namespace Chaos::Softs
