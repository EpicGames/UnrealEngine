// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/GeometryParticlesfwd.h"  // for EGeometryParticlesSimType

namespace UE::Math
{
	template<typename T> struct TTransform;
}

class FPerSolverFieldSystem;

namespace Chaos
{
	// Chaos types
	template <typename T, int d> class TVector;
	template<class T, int m, int n> class PMatrix;
	template <class T, int d> class TPBDParticles;
	template <class T, int d> class TRotation;
	template <class T, int d> class TRigidTransform;
	template <class T, int d> class TAABB;
	template <class T> struct TVector3AndScalar;

	class FPerParticleGravity;

	// Softs types
	namespace Softs
	{
		// Softs solver types
		//using FSolverReal = FReal;
		using FSolverReal = FRealSingle;
		using FSolverVec2 = TVector<FSolverReal, 2>;
		using FSolverVec3 = TVector<FSolverReal, 3>;
		using FSolverMatrix22 = PMatrix<FSolverReal, 2, 2>;
		using FSolverMatrix33 = PMatrix<FSolverReal, 3, 3>;
		using FSolverMatrix44 = PMatrix<FSolverReal, 4, 4>;
		class FSolverCollisionParticles;
		using FSolverRigidParticles UE_DEPRECATED(5.4, "Alias FSolverRigidParticles has been renamed FSolverCollisionParticles and refers to a different class.") = FSolverCollisionParticles;
		using FSolverRotation3 = TRotation<FSolverReal, 3>;
		using FSolverRigidTransform3 = TRigidTransform<FSolverReal, 3>;
		using FSolverTransform3 = UE::Math::TTransform<FSolverReal>;
		using FSolverAABB3 = TAABB<FSolverReal, 3>;
		class FSolverParticles;
		class FSolverParticlesRange;
		class FSolverCollisionParticlesRange;
		struct FPAndInvM;

		// Softs solver class
		class FPBDEvolution;
		class FEvolution;

		// Softs solver constraint classes
		class FPBDSpringConstraints;
		class FPBDEdgeSpringConstraints;
		class FXPBDStretchBiasElementConstraints;
		class FXPBDAnisotropicSpringConstraints;
		class FPBDBendingSpringConstraints;
		class FXPBDSpringConstraints;
		class FXPBDEdgeSpringConstraints;
		class FXPBDBendingSpringConstraints;
		class FPBDBendingConstraints;
		class FXPBDBendingConstraints;
		class FXPBDAnisotropicBendingConstraints;
		class FPBDAxialSpringConstraints;
		class FXPBDAxialSpringConstraints;
		class FPBDAreaSpringConstraints;
		class FXPBDAreaSpringConstraints;
		class FPBDVolumeConstraint;
		class FPBDLongRangeConstraints;
		class FXPBDLongRangeConstraints;
		class FPBDSphericalConstraint;
		class FPBDSphericalBackstopConstraint;
		class FPBDAnimDriveConstraint;
		class FPBDShapeConstraints;
		class FPBDCollisionSpringConstraints;
		class FPBDSkinnedTriangleMeshCollisionConstraints;
		class FPBDTriangleMeshIntersections;
		class FPBDTriangleMeshCollisions;
		class FPBDSelfCollisionSphereConstraints;
		class FPBDSoftBodyCollisionConstraint;
		class FMultiResConstraints;
		class FXPBDVertexConstraints;
		class FXPBDVertexFaceConstraints;
		class FXPBDFaceConstraints;
		class FPBDVertexFaceRepulsionConstraints;

		// Softs solver forces
		class FVelocityAndPressureField;
		class FExternalForces;

		// Linear system solver
		class FEvolutionLinearSystem;

		// Property facade
		class FCollectionPropertyConstFacade;
	}  // End namespace Softs
}  // End namespace Chaos
