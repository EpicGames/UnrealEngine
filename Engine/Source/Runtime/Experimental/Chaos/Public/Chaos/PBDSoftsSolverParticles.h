// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Real.h"
#include "Chaos/Vector.h"
#include "Chaos/ArrayCollectionArray.h"
#include "Chaos/DynamicParticles.h"
#include "Chaos/PBDSoftsEvolutionFwd.h"

namespace Chaos::Softs
{

struct FPAndInvM
{
	FSolverVec3 P;
	FSolverReal InvM;
};

class FSolverParticles : public TDynamicParticles<FSolverReal, 3>
{
public:
	typedef TDynamicParticles<FSolverReal, 3> Base;

	FSolverParticles()
	    : Base()
	{
		TArrayCollection::AddArray(&MPAndInvM);
	}
	FSolverParticles(const FSolverParticles& Other) = delete;
	FSolverParticles(FSolverParticles&& Other)
	    : Base(MoveTemp(Other)), MPAndInvM(MoveTemp(Other.MPAndInvM))
	{
		TArrayCollection::AddArray(&MPAndInvM);
	}

	const FSolverVec3& P(const int32 index) const { return MPAndInvM[index].P; }
	FSolverVec3& P(const int32 index) { return MPAndInvM[index].P; }
	const FSolverVec3& X(const int32 index) const { 
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return Base::X(index); 
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	FSolverVec3& X(const int32 index) { 
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return Base::X(index); 
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	const FSolverVec3& GetP(const int32 index) const { return MPAndInvM[index].P; }
	void SetP(const int32 index, const FSolverVec3& InP) { MPAndInvM[index].P = InP; }
	// When PBD is solving, V = last time step's V and we use the latest P to estimate velocity, (VelocityEstimate = (P - X)/Dt.
	// After PBD (e.g., during implicit timestepping CG solve), V = current velocity estimate, VPrev = last time step's velocity
	// Warning: VPrev and Acceleration use the same arrays as Acceleration is only used by PBD and VPrev is only used by the linear solver
	const FSolverVec3& VPrev(const int32 Index) const { return Acceleration(Index); }
	FSolverVec3& VPrev(const int32 Index) { return Acceleration(Index); }

	const FPAndInvM& PAndInvM(const int32 index) const { return MPAndInvM[index]; }
	FPAndInvM& PAndInvM(const int32 index) { return MPAndInvM[index]; }

	TArrayCollectionArray<FPAndInvM>& GetPAndInvM() { return MPAndInvM; }
	const TArrayCollectionArray<FPAndInvM>& GetPAndInvM() const { return MPAndInvM; }
	// Warning: VPrev and Acceleration use the same arrays as Acceleration is only used by PBD and VPrev is only used by the linear solver
	TArrayCollectionArray<FSolverVec3>& GetVPrev() { return GetAcceleration(); }
	const TArrayCollectionArray<FSolverVec3>& GetVPrev() const { return  GetAcceleration(); }

	// Convenience method to match interface of FSolverParticlesRange
	template<typename T>
	TConstArrayView<T> GetConstArrayView(const TArray<T>& Data) const { return TConstArrayView<T>(Data); }
	template<typename T>
	TArrayView<T> GetArrayView(const TArray<T>& Data) { return TArrayView<T>(Data); }

private:
	// Note that InvM is copied from TDynamicParticles to here during ApplyPreSimulationTransforms (when P is updated)
	TArrayCollectionArray<FPAndInvM> MPAndInvM;

};

}
