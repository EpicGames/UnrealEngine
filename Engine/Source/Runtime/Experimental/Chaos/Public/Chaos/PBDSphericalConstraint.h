// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/PBDSoftsEvolutionFwd.h"
#include "Chaos/PBDSoftsSolverParticles.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "Chaos/SoftsSolverParticlesRange.h"
#include "ChaosStats.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainersFwd.h"
#include "Chaos/Framework/Parallel.h"

DECLARE_CYCLE_STAT(TEXT("Chaos PBD Spherical Constraints"), STAT_PBD_Spherical, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos PBD Spherical Backstop Constraints"), STAT_PBD_SphericalBackstop, STATGROUP_Chaos);

#if !defined(CHAOS_SPHERICAL_ISPC_ENABLED_DEFAULT)
#define CHAOS_SPHERICAL_ISPC_ENABLED_DEFAULT 1
#endif

#if !defined(USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING)
#define USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING 0
#endif

// Support run-time toggling on supported platforms in non-shipping configurations
#if !INTEL_ISPC || (UE_BUILD_SHIPPING && !USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING)
static constexpr bool bChaos_Spherical_ISPC_Enabled = INTEL_ISPC && CHAOS_SPHERICAL_ISPC_ENABLED_DEFAULT;
#else
extern CHAOS_API bool bChaos_Spherical_ISPC_Enabled;
#endif

namespace Chaos::Softs
{

class FPBDSphericalConstraint final
{
public:
	static bool IsEnabled(const FCollectionPropertyConstFacade& PropertyCollection)
	{
		return IsMaxDistanceEnabled(PropertyCollection, false);
	}

	FPBDSphericalConstraint(
		const uint32 InParticleOffset,
		const uint32 InParticleCount,
		const TArray<FSolverVec3>& InAnimationPositions,  // Use global indexation (will need adding ParticleOffset)
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,  // Use local indexation
		const FCollectionPropertyConstFacade& PropertyCollection,
		FSolverReal MeshScale
	)
		: AnimationPositions(InAnimationPositions)
		, SphereRadii(WeightMaps.FindRef(GetMaxDistanceString(PropertyCollection, MaxDistanceName.ToString())))
		, ParticleOffset(InParticleOffset)
		, ParticleCount(InParticleCount)
		, Scale(MeshScale)
		, MaxDistanceBase((FSolverReal)GetLowMaxDistance(PropertyCollection, 0.f))
		, MaxDistanceRange((FSolverReal)GetHighMaxDistance(PropertyCollection, 1.f) - MaxDistanceBase)
		, MaxDistanceIndex(PropertyCollection)
	{
	}

	FPBDSphericalConstraint(
		const uint32 InParticleOffset,
		const uint32 InParticleCount,
		const TArray<FSolverVec3>& InAnimationPositions,  // Use global indexation (will need adding ParticleOffset)
		const TConstArrayView<FRealSingle>& InSphereRadii  // Use local indexation
	)
		: AnimationPositions(InAnimationPositions)
		, SphereRadii(InSphereRadii)
		, ParticleOffset(InParticleOffset)
		, ParticleCount(InParticleCount)
		, Scale((FSolverReal)1.)
		, MaxDistanceIndex(ForceInit)
	{
	}

	~FPBDSphericalConstraint() = default;

	CHAOS_API void SetProperties(
		const FCollectionPropertyConstFacade& PropertyCollection,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
		FSolverReal MeshScale);

	template<typename SolverParticlesOrRange>
	void Apply(SolverParticlesOrRange& Particles, const FSolverReal Dt) const
	{
		SCOPE_CYCLE_COUNTER(STAT_PBD_Spherical);

		if (bRealTypeCompatibleWithISPC && bChaos_Spherical_ISPC_Enabled)
		{
			ApplyHelperISPC(Particles, Dt);
		}
		else
		{
			if (SphereRadii.Num() == ParticleCount)
			{
				constexpr bool bHasMaxDistance = true;
				ApplyHelper<bHasMaxDistance>(Particles, Dt);
			}
			else
			{
				constexpr bool bHasMaxDistance = false;
				ApplyHelper<bHasMaxDistance>(Particles, Dt);
			}
		}
	}

	// Set a new mesh scale
	void SetScale(FSolverReal InScale) { Scale = InScale; }
	FSolverReal GetScale() const { return Scale; }

	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(MaxDistance, float);

private:

	template<bool bHasMaxDistance, typename SolverParticlesOrRange>
	void ApplyHelper(SolverParticlesOrRange& Particles, const FSolverReal Dt) const
	{
		const TConstArrayView<FSolverVec3> AnimationPositionsView = Particles.GetConstArrayView(AnimationPositions);
		PhysicsParallelFor(ParticleCount, [this, &Particles, Dt, &AnimationPositionsView](int32 Index)  // TODO: profile need for parallel loop based on particle count
		{
			const int32 ParticleIndex = ParticleOffset + Index;

			if (Particles.InvM(ParticleIndex) == 0)
			{
				return;
			}

			const FSolverReal Radius = (bHasMaxDistance ? MaxDistanceBase + MaxDistanceRange * SphereRadii[Index] : MaxDistanceBase) * Scale;
			const FSolverVec3& Center = AnimationPositionsView[ParticleIndex];

			const FSolverVec3 CenterToParticle = Particles.P(ParticleIndex) - Center;
			const FSolverReal DistanceSquared = CenterToParticle.SizeSquared();

			static const FSolverReal DeadZoneSquareRadius = UE_SMALL_NUMBER; // We will not push the particle away in the dead zone
			if (DistanceSquared > FMath::Square(Radius) + DeadZoneSquareRadius)
			{
				const FSolverReal Distance = FMath::Sqrt(DistanceSquared);
				const FSolverVec3 PositionOnSphere = (Radius / Distance) * CenterToParticle;
				Particles.P(ParticleIndex) = Center + PositionOnSphere;
			}
		});
	}

	template<typename SolverParticlesOrRange>
	CHAOS_API void ApplyHelperISPC(SolverParticlesOrRange& Particles, const FSolverReal Dt) const;

protected:
	const TArray<FSolverVec3>& AnimationPositions;  // Use global indexation (will need adding ParticleOffset)
	TConstArrayView<FRealSingle> SphereRadii;  // Use local indexation
	const int32 ParticleOffset;
	const int32 ParticleCount;

private:
	FSolverReal Scale = (FSolverReal)1.;
	FSolverReal MaxDistanceBase = (FSolverReal)0.;
	FSolverReal MaxDistanceRange = (FSolverReal)1.;

	UE_CHAOS_DECLARE_INDEXED_PROPERTYCOLLECTION_NAME(MaxDistance, float);
};

class FPBDSphericalBackstopConstraint final
{
public:
	static bool IsEnabled(const FCollectionPropertyConstFacade& PropertyCollection)
	{
		return IsBackstopRadiusEnabled(PropertyCollection, false) ||  // Radius makes more sense than distance to enable the constraint here, since without any radius there isn't a backstop
			IsBackstopRadiusAnimatable(PropertyCollection, false);  // Backstop can be re-enabled if animated
	}

	FPBDSphericalBackstopConstraint(
		const int32 InParticleOffset,
		const int32 InParticleCount,
		const TArray<FSolverVec3>& InAnimationPositions,
		const TArray<FSolverVec3>& InAnimationNormals,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,  // Use local indexation
		const FCollectionPropertyConstFacade& PropertyCollection,
		FSolverReal MeshScale, 
		bool bInUseGlobalIndexation = true   // Use global indexation (will need adding ParticleOffset)
	)
		: AnimationPositions(InAnimationPositions)
		, AnimationNormals(InAnimationNormals)
		, SphereRadii(WeightMaps.FindRef(GetBackstopRadiusString(PropertyCollection, BackstopRadiusName.ToString())))
		, SphereOffsetDistances(WeightMaps.FindRef(GetBackstopDistanceString(PropertyCollection, BackstopDistanceName.ToString())))
		, ParticleOffset(InParticleOffset)
		, ParticleCount(InParticleCount)
		, Scale(MeshScale)
		, BackstopRadiusBase((FSolverReal)FMath::Max(GetLowBackstopRadius(PropertyCollection, 0.f), 0.f))
		, BackstopRadiusRange((FSolverReal)FMath::Max(GetHighBackstopRadius(PropertyCollection, 1.f), 0.f) - BackstopRadiusBase)
		, BackstopDistanceBase((FSolverReal)GetLowBackstopDistance(PropertyCollection, 0.f))
		, BackstopDistanceRange((FSolverReal)GetHighBackstopDistance(PropertyCollection, 1.f) - BackstopDistanceBase)
		, bUseLegacyBackstop(GetUseLegacyBackstop(PropertyCollection, false))  // Only set the legacy backstop in constructor
		, bMeshUseGlobalIndexation(bInUseGlobalIndexation)
		, BackstopDistanceIndex(PropertyCollection)
		, BackstopRadiusIndex(PropertyCollection)
		, UseLegacyBackstopIndex(PropertyCollection)
	{
	}

	FPBDSphericalBackstopConstraint(
		const int32 InParticleOffset,
		const int32 InParticleCount,
		const TArray<FSolverVec3>& InAnimationPositions,
		const TArray<FSolverVec3>& InAnimationNormals,
		const TConstArrayView<FRealSingle>& InSphereRadii,  // Use local indexation
		const TConstArrayView<FRealSingle>& InSphereOffsetDistances,  // Use local indexation
		const bool bInUseLegacyBackstop,  // Do not include the sphere radius in the distance calculations when this is true, 
		bool bInUseGlobalIndexation = true  // Use global indexation (will need adding ParticleOffset)
	)
		: AnimationPositions(InAnimationPositions)
		, AnimationNormals(InAnimationNormals)
		, SphereRadii(InSphereRadii)
		, SphereOffsetDistances(InSphereOffsetDistances)
		, ParticleOffset(InParticleOffset)
		, ParticleCount(InParticleCount)
		, Scale((FSolverReal)1.)
		, bEnabled(true)
		, bUseLegacyBackstop(bInUseLegacyBackstop)
		, bMeshUseGlobalIndexation(bInUseGlobalIndexation)
		, BackstopDistanceIndex(ForceInit)
		, BackstopRadiusIndex(ForceInit)
		, UseLegacyBackstopIndex(ForceInit)
	{
	}
	~FPBDSphericalBackstopConstraint() = default;

	CHAOS_API void SetProperties(
		const FCollectionPropertyConstFacade& PropertyCollection,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
		FSolverReal MeshScale);

	void SetEnabled(bool bInEnabled) { bEnabled = bInEnabled; }
	bool IsEnabled() const { return bEnabled; }

	void Apply(FSolverParticles& Particles, const FSolverReal Dt) const
	{
		FSolverParticlesRange ParticlesAsRange(&Particles, ParticleOffset, ParticleCount);
		return Apply(ParticlesAsRange, Dt);
	}

	void Apply(FSolverParticlesRange& Particles, const FSolverReal Dt) const
	{
		SCOPE_CYCLE_COUNTER(STAT_PBD_SphericalBackstop);

		if (bEnabled)
		{
			if (bUseLegacyBackstop)
			{
				// SphereOffsetDistances includes the sphere radius
				// This is harder to author, and does not follow the NvCloth specs.
				// However, this is how it's been done in the Unreal Engine PhysX cloth implementation.
				if (bRealTypeCompatibleWithISPC && bChaos_Spherical_ISPC_Enabled)
				{
					ApplyLegacyHelperISPC(Particles, Dt);
				}
				else if (SphereRadii.Num() == ParticleCount)
				{
					if (SphereOffsetDistances.Num() == ParticleCount)
					{
						constexpr bool bHasBackstopDistance = true;
						constexpr bool bHasBackstopRadius = true;
						ApplyLegacyHelper<bHasBackstopDistance, bHasBackstopRadius>(Particles, Dt);
					}
					else
					{
						constexpr bool bHasBackstopDistance = false;
						constexpr bool bHasBackstopRadius = true;
						ApplyLegacyHelper<bHasBackstopDistance, bHasBackstopRadius>(Particles, Dt);
					}
				}
				else if (SphereOffsetDistances.Num() == ParticleCount)
				{
					constexpr bool bHasBackstopDistance = true;
					constexpr bool bHasBackstopRadius = false;
					ApplyLegacyHelper<bHasBackstopDistance, bHasBackstopRadius>(Particles, Dt);
				}
				else
				{
					constexpr bool bHasBackstopDistance = false;
					constexpr bool bHasBackstopRadius = false;
					ApplyLegacyHelper<bHasBackstopDistance, bHasBackstopRadius>(Particles, Dt);
				}
			}
			else
			{
				// SphereOffsetDistances doesn't include the sphere radius
				if (bRealTypeCompatibleWithISPC && bChaos_Spherical_ISPC_Enabled)
				{
					ApplyHelperISPC(Particles, Dt);
				}
				else if (SphereRadii.Num() == ParticleCount)
				{
					if (SphereOffsetDistances.Num() == ParticleCount)
					{
						constexpr bool bHasBackstopDistance = true;
						constexpr bool bHasBackstopRadius = true;
						ApplyHelper<bHasBackstopDistance, bHasBackstopRadius>(Particles, Dt);
					}
					else
					{
						constexpr bool bHasBackstopDistance = false;
						constexpr bool bHasBackstopRadius = true;
						ApplyHelper<bHasBackstopDistance, bHasBackstopRadius>(Particles, Dt);
					}
				}
				else if (SphereOffsetDistances.Num() == ParticleCount)
				{
					constexpr bool bHasBackstopDistance = true;
					constexpr bool bHasBackstopRadius = false;
					ApplyHelper<bHasBackstopDistance, bHasBackstopRadius>(Particles, Dt);
				}
				else
				{
					constexpr bool bHasBackstopDistance = false;
					constexpr bool bHasBackstopRadius = false;
					ApplyHelper<bHasBackstopDistance, bHasBackstopRadius>(Particles, Dt);
				}
			}
		}
	}

	// Set a new mesh scale
	void SetScale(FSolverReal InScale) { Scale = InScale; }
	FSolverReal GetScale() const { return Scale; }

	bool UseLegacyBackstop() const
	{
		return bUseLegacyBackstop;
	}

	FSolverReal GetBackstopRadius(int32 ConstraintIndex) const
	{
		return SphereRadii.Num() == ParticleCount ? BackstopRadiusBase + BackstopRadiusRange * SphereRadii[ConstraintIndex] : BackstopRadiusBase;
	}

	FSolverReal GetBackstopDistance(int32 ConstraintIndex) const
	{
		return SphereOffsetDistances.Num() == ParticleCount ? BackstopDistanceBase + BackstopDistanceRange * SphereOffsetDistances[ConstraintIndex] : BackstopDistanceBase;
	}

	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(BackstopDistance, float);
	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(BackstopRadius, float);
	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(UseLegacyBackstop, bool);
	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(BackstopMeshName, bool); // Actually just using for string name

private:

	template<bool bHasBackstopDistance, bool bHasBackstopRadius>
	void ApplyHelper(FSolverParticlesRange& Particles, const FSolverReal Dt) const
	{
		const TConstArrayView<FSolverVec3> AnimationPositionsView = bMeshUseGlobalIndexation ? Particles.GetConstArrayView(AnimationPositions) : TConstArrayView<FSolverVec3>(AnimationPositions);
		const TConstArrayView<FSolverVec3> AnimationNormalsView = bMeshUseGlobalIndexation ? Particles.GetConstArrayView(AnimationNormals) : TConstArrayView<FSolverVec3>(AnimationNormals);
		PhysicsParallelFor(ParticleCount, [this, &Particles, Dt, &AnimationPositionsView, &AnimationNormalsView](int32 ParticleIndex)  // TODO: profile need for parallel loop based on particle count
		{
			if (Particles.InvM(ParticleIndex) == 0)
			{
				return;
			}

			const FSolverVec3& AnimationPosition = AnimationPositionsView[ParticleIndex];
			const FSolverVec3& AnimationNormal = AnimationNormalsView[ParticleIndex];

			const FSolverReal SphereOffsetDistance = (bHasBackstopDistance ? BackstopDistanceBase + BackstopDistanceRange * SphereOffsetDistances[ParticleIndex] : BackstopDistanceBase) * Scale;
			const FSolverReal Radius = (bHasBackstopRadius ? BackstopRadiusBase + BackstopRadiusRange * SphereRadii[ParticleIndex] : BackstopRadiusBase) * Scale;

			const FSolverVec3 Center = AnimationPosition - (Radius + SphereOffsetDistance) * AnimationNormal;  // Non legacy version adds radius to the distance
			const FSolverVec3 CenterToParticle = Particles.P(ParticleIndex) - Center;
			const FSolverReal DistanceSquared = CenterToParticle.SizeSquared();

			static const FSolverReal DeadZoneSquareRadius = UE_SMALL_NUMBER;
			if (DistanceSquared < DeadZoneSquareRadius)
			{
				Particles.P(ParticleIndex) = AnimationPosition - SphereOffsetDistance * AnimationNormal;  // Non legacy version adds radius to the distance
			}
			else if (DistanceSquared < FMath::Square(Radius))
			{
				const FSolverVec3 PositionOnSphere = (Radius / sqrt(DistanceSquared)) * CenterToParticle;
				Particles.P(ParticleIndex) = Center + PositionOnSphere;
			}
			// Else the particle is outside the sphere, and there is nothing to do
		});
	}

	template<bool bHasBackstopDistance, bool bHasBackstopRadius>
	void ApplyLegacyHelper(FSolverParticlesRange& Particles, const FSolverReal Dt) const
	{
		const TConstArrayView<FSolverVec3> AnimationPositionsView = bMeshUseGlobalIndexation ? Particles.GetConstArrayView(AnimationPositions) : TConstArrayView<FSolverVec3>(AnimationPositions);
		const TConstArrayView<FSolverVec3> AnimationNormalsView = bMeshUseGlobalIndexation ? Particles.GetConstArrayView(AnimationNormals) : TConstArrayView<FSolverVec3>(AnimationNormals);
		PhysicsParallelFor(ParticleCount, [this, &Particles, Dt, &AnimationPositionsView, &AnimationNormalsView](int32 ParticleIndex)  // TODO: profile need for parallel loop based on particle count
		{
			if (Particles.InvM(ParticleIndex) == 0)
			{
				return;
			}

			const FSolverVec3& AnimationPosition = AnimationPositionsView[ParticleIndex];
			const FSolverVec3& AnimationNormal = AnimationNormalsView[ParticleIndex];

			const FSolverReal SphereOffsetDistance = (bHasBackstopDistance ? BackstopDistanceBase + BackstopDistanceRange * SphereOffsetDistances[ParticleIndex] : BackstopDistanceBase) * Scale;
			const FSolverReal Radius = (bHasBackstopRadius ? BackstopRadiusBase + BackstopRadiusRange * SphereRadii[ParticleIndex] : BackstopRadiusBase) * Scale;

			const FSolverVec3 Center = AnimationPosition - SphereOffsetDistance * AnimationNormal;  // Legacy version already includes the radius within the distance
			const FSolverVec3 CenterToParticle = Particles.P(ParticleIndex) - Center;
			const FSolverReal DistanceSquared = CenterToParticle.SizeSquared();

			static const FSolverReal DeadZoneSquareRadius = UE_SMALL_NUMBER;
			if (DistanceSquared < DeadZoneSquareRadius)
			{
				Particles.P(ParticleIndex) = AnimationPosition - (SphereOffsetDistance - Radius) * AnimationNormal;  // Legacy version already includes the radius to the distance
			}
			else if (DistanceSquared < FMath::Square(Radius))
			{
				const FSolverVec3 PositionOnSphere = (Radius / sqrt(DistanceSquared)) * CenterToParticle;
				Particles.P(ParticleIndex) = Center + PositionOnSphere;
			}
			// Else the particle is outside the sphere, and there is nothing to do
		});
	}

	CHAOS_API void ApplyLegacyHelperISPC(FSolverParticlesRange& Particles, const FSolverReal Dt) const;
	CHAOS_API void ApplyHelperISPC(FSolverParticlesRange& Particles, const FSolverReal Dt) const;

private:
	const TArray<FSolverVec3>& AnimationPositions;  // Positions of spheres
	const TArray<FSolverVec3>& AnimationNormals; // Sphere offset directions
	TConstArrayView<FRealSingle> SphereRadii; // Start at index 0, use local indexation
	TConstArrayView<FRealSingle> SphereOffsetDistances;  // Sphere position offsets, use local indexation
	const int32 ParticleOffset;
	const int32 ParticleCount;
	FSolverReal Scale = (FSolverReal)1.;
	FSolverReal BackstopRadiusBase = (FSolverReal)0.;
	FSolverReal BackstopRadiusRange = (FSolverReal)1.;
	FSolverReal BackstopDistanceBase = (FSolverReal)0.;
	FSolverReal BackstopDistanceRange = (FSolverReal)1.;
	bool bEnabled = true;
	bool bUseLegacyBackstop = false;
	bool bMeshUseGlobalIndexation = true; // use global indexation (will need adding ParticleOffset)

	UE_CHAOS_DECLARE_INDEXED_PROPERTYCOLLECTION_NAME(BackstopDistance, float);
	UE_CHAOS_DECLARE_INDEXED_PROPERTYCOLLECTION_NAME(BackstopRadius, float);
	UE_CHAOS_DECLARE_INDEXED_PROPERTYCOLLECTION_NAME(UseLegacyBackstop, bool);
};

}  // End namespace Chaos::Softs
