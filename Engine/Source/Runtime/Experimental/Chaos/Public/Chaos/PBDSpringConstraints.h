// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/PBDSpringConstraintsBase.h"
#include "Chaos/CollectionPropertyFacade.h"

namespace Chaos
{
	class FTriangleMesh;
}

namespace Chaos::Softs
{

class FPBDSpringConstraints : public FPBDSpringConstraintsBase
{
public:
	template<int32 Valence>
	FPBDSpringConstraints(
		const FSolverParticlesRange& Particles,
		const TArray<TVector<int32, Valence>>& InConstraints,
		const TConstArrayView<FRealSingle>& StiffnessMultipliers,
		const FSolverVec2& InStiffness,
		bool bTrimKinematicConstraints = false,
		bool bInitColor = true,
		typename TEnableIf<Valence >= 2 && Valence <= 4>::Type* = nullptr)
		: Base(
			Particles,
			InConstraints,
			StiffnessMultipliers,
			InStiffness,
			bTrimKinematicConstraints)
	{
		if (bInitColor)
		{
			InitColor(Particles);
		}
	}

	template<int32 Valence>
	FPBDSpringConstraints(
		const FSolverParticles& Particles,
		int32 InParticleOffset,
		int32 InParticleCount,
		const TArray<TVector<int32, Valence>>& InConstraints,
		const TConstArrayView<FRealSingle>& StiffnessMultipliers,
		const FSolverVec2& InStiffness,
		bool bTrimKinematicConstraints = false,
		bool bInitColor = true,
		typename TEnableIf<Valence >= 2 && Valence <= 4>::Type* = nullptr)
		: Base(
			Particles,
			InParticleOffset,
			InParticleCount,
			InConstraints,
			StiffnessMultipliers,
			InStiffness,
			bTrimKinematicConstraints)
	{
		if (bInitColor)
		{
			InitColor(Particles);
		}
	}

	virtual ~FPBDSpringConstraints() override = default;

	template<typename SolverParticlesOrRange>
	CHAOS_API void Apply(SolverParticlesOrRange& Particles, const FSolverReal Dt) const;

	const TArray<int32>& GetConstraintsPerColorStartIndex() const { return ConstraintsPerColorStartIndex; }

protected:

	/* Returns OrigToReorderedVertices */
	template<typename SolverParticlesOrRange>
	CHAOS_API TArray<int32> InitColor(const SolverParticlesOrRange& InParticles);


	typedef FPBDSpringConstraintsBase Base;
	using Base::Constraints;
	using Base::Stiffness;
	using Base::ParticleOffset;
	using Base::ParticleCount;

private:
	template<typename SolverParticlesOrRange>
	void ApplyHelper(SolverParticlesOrRange& Particles, const FSolverReal Dt, const int32 ConstraintIndex, const FSolverReal ExpStiffnessValue) const;

private:
	TArray<int32> ConstraintsPerColorStartIndex; // Constraints are ordered so each batch is contiguous. This is ColorNum + 1 length so it can be used as start and end.
};

class FPBDEdgeSpringConstraints final : public FPBDSpringConstraints
{
public:
	static bool IsEnabled(const FCollectionPropertyConstFacade& PropertyCollection)
	{
		return IsEdgeSpringStiffnessEnabled(PropertyCollection, false);
	}

	FPBDEdgeSpringConstraints(
		const FSolverParticlesRange& Particles,
		const TArray<TVec3<int32>>& InConstraints,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
		const FCollectionPropertyConstFacade& PropertyCollection,
		bool bTrimKinematicConstraints = false)
		: FPBDSpringConstraints(
			Particles,
			InConstraints,
			WeightMaps.FindRef(GetEdgeSpringStiffnessString(PropertyCollection, EdgeSpringStiffnessName.ToString())),
			FSolverVec2(GetWeightedFloatEdgeSpringStiffness(PropertyCollection, 1.f)),
			bTrimKinematicConstraints
		)
		, WarpScale(FSolverVec2(1.f))
		, WeftScale(FSolverVec2(1.f))
		, EdgeSpringStiffnessIndex(PropertyCollection)
		, EdgeSpringWarpScaleIndex(ForceInit)
		, EdgeSpringWeftScaleIndex(ForceInit)
	{
	}

	FPBDEdgeSpringConstraints(
		const FSolverParticles& Particles,
		int32 ParticleOffset,
		int32 ParticleCount,
		const TArray<TVec3<int32>>& InConstraints,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
		const FCollectionPropertyConstFacade& PropertyCollection,
		bool bTrimKinematicConstraints = false)
		: FPBDSpringConstraints(
			Particles,
			ParticleOffset,
			ParticleCount,
			InConstraints,
			WeightMaps.FindRef(GetEdgeSpringStiffnessString(PropertyCollection, EdgeSpringStiffnessName.ToString())),
			FSolverVec2(GetWeightedFloatEdgeSpringStiffness(PropertyCollection, 1.f)),
			bTrimKinematicConstraints)
		, WarpScale(FSolverVec2(1.f))
		, WeftScale(FSolverVec2(1.f))
		, EdgeSpringStiffnessIndex(PropertyCollection)
		, EdgeSpringWarpScaleIndex(ForceInit)
		, EdgeSpringWeftScaleIndex(ForceInit)
	{
	}

	/** This version of the constructor supports Warp and Weft Scale */
	CHAOS_API FPBDEdgeSpringConstraints(
		const FSolverParticlesRange& Particles,
		const FTriangleMesh& TriangleMesh,
		const TArray<TVec3<FVec2f>>& FaceVertexPatternPositions,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
		const FCollectionPropertyConstFacade& PropertyCollection,
		bool bTrimKinematicConstraints = false);

	virtual ~FPBDEdgeSpringConstraints() override = default;

	CHAOS_API void SetProperties(
		const FCollectionPropertyConstFacade& PropertyCollection,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps);

	CHAOS_API void ApplyProperties(const FSolverReal Dt, const int32 NumIterations);

	CHAOS_API void ResetRestLengths(const TConstArrayView<FSolverVec3>& Positions);

	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(EdgeSpringStiffness, float);
	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(EdgeSpringWarpScale, float);
	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(EdgeSpringWeftScale, float);

private:

	void InitFromPatternData(const TArray<TVec3<FVec2f>>& FaceVertexPatternPositions, const FTriangleMesh& TriangleMesh);
	void UpdateDists();

	using FPBDSpringConstraints::ParticleCount;

	bool bWarpWeftScaleEnabled = false;
	FPBDWeightMap WarpScale;
	FPBDWeightMap WeftScale;
	TArray<FSolverReal> BaseDists; // Without Warp/Weft Scale applied
	TArray<FSolverVec2> WarpWeftScaleBaseMultipliers;

	UE_CHAOS_DECLARE_INDEXED_PROPERTYCOLLECTION_NAME(EdgeSpringStiffness, float);
	UE_CHAOS_DECLARE_INDEXED_PROPERTYCOLLECTION_NAME(EdgeSpringWarpScale, float);
	UE_CHAOS_DECLARE_INDEXED_PROPERTYCOLLECTION_NAME(EdgeSpringWeftScale, float);
};

class FPBDBendingSpringConstraints final : public FPBDSpringConstraints
{
public:
	static bool IsEnabled(const FCollectionPropertyConstFacade& PropertyCollection)
	{
		return IsBendingSpringStiffnessEnabled(PropertyCollection, false);
	}

	FPBDBendingSpringConstraints(
		const FSolverParticlesRange& Particles,
		const TArray<TVec2<int32>>& InConstraints,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
		const FCollectionPropertyConstFacade& PropertyCollection,
		bool bTrimKinematicConstraints = false)
		: FPBDSpringConstraints(
			Particles,
			InConstraints,
			WeightMaps.FindRef(GetBendingSpringStiffnessString(PropertyCollection, BendingSpringStiffnessName.ToString())),
			FSolverVec2(GetWeightedFloatBendingSpringStiffness(PropertyCollection, 1.f)),
			bTrimKinematicConstraints)
		, BendingSpringStiffnessIndex(PropertyCollection)
	{
		InitColor(Particles);
	}

	FPBDBendingSpringConstraints(
		const FSolverParticles& Particles,
		int32 ParticleOffset,
		int32 ParticleCount,
		const TArray<TVec2<int32>>& InConstraints,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
		const FCollectionPropertyConstFacade& PropertyCollection,
		bool bTrimKinematicConstraints = false)
		: FPBDSpringConstraints(
			Particles,
			ParticleOffset,
			ParticleCount,
			InConstraints,
			WeightMaps.FindRef(GetBendingSpringStiffnessString(PropertyCollection, BendingSpringStiffnessName.ToString())),
			FSolverVec2(GetWeightedFloatBendingSpringStiffness(PropertyCollection, 1.f)),
			bTrimKinematicConstraints)
		, BendingSpringStiffnessIndex(PropertyCollection)
	{
		InitColor(Particles);
	}

	virtual ~FPBDBendingSpringConstraints() override = default;

	CHAOS_API void SetProperties(
		const FCollectionPropertyConstFacade& PropertyCollection,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps);

	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(BendingSpringStiffness, float);

private:
	using FPBDSpringConstraints::ParticleCount;

	UE_CHAOS_DECLARE_INDEXED_PROPERTYCOLLECTION_NAME(BendingSpringStiffness, float);
};

}  // End namespace Chaos::Softs

#if !defined(CHAOS_SPRING_ISPC_ENABLED_DEFAULT)
#define CHAOS_SPRING_ISPC_ENABLED_DEFAULT 1
#endif

#if !defined(USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING)
#define USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING 0
#endif

// Support run-time toggling on supported platforms in non-shipping configurations
#if !INTEL_ISPC || (UE_BUILD_SHIPPING && !USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING)
static constexpr bool bChaos_Spring_ISPC_Enabled = INTEL_ISPC && CHAOS_SPRING_ISPC_ENABLED_DEFAULT;
#else
extern CHAOS_API bool bChaos_Spring_ISPC_Enabled;
#endif
