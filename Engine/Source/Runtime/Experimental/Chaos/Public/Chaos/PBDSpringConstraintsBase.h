// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/PBDSoftsEvolutionFwd.h"
#include "Chaos/Utilities.h"
#include "Chaos/PBDSoftsSolverParticles.h"
#include "Chaos/SoftsSolverParticlesRange.h"
#include "Chaos/PBDStiffness.h"
#include "Containers/Array.h"
#include "Templates/EnableIf.h"

namespace Chaos::Softs
{

class FPBDSpringConstraintsBase
{
public:
	template<int32 Valence UE_REQUIRES(Valence >= 2 && Valence <= 4)>
	FPBDSpringConstraintsBase(
		const FSolverParticlesRange& Particles,
		const TArray<TVector<int32, Valence>>& InConstraints,
		const TConstArrayView<FRealSingle>& StiffnessMultipliers,
		const FSolverVec2& InStiffness,
		bool bTrimKinematicConstraints = false,
		FSolverReal MaxStiffness = FPBDStiffness::DefaultPBDMaxStiffness)
		: Constraints(TrimConstraints(InConstraints,
			[&Particles, bTrimKinematicConstraints](int32 Index0, int32 Index1)
			{
				return bTrimKinematicConstraints && Particles.InvM(Index0) == (FSolverReal)0. && Particles.InvM(Index1) == (FSolverReal)0.;
			}))
		, ParticleOffset(0)
		, ParticleCount(Particles.GetRangeSize())
		, Stiffness(
			InStiffness,
			StiffnessMultipliers,
			TConstArrayView<TVec2<int32>>(Constraints),
			ParticleOffset,
			ParticleCount,
			FPBDStiffness::DefaultTableSize,
			FPBDStiffness::DefaultParameterFitBase,
			MaxStiffness)
	{
		// Update distances
		Dists.SetNumUninitialized(Constraints.Num());
		CalculateRestLengths(Particles.XArray(), Dists);
	}

	template<int32 Valence UE_REQUIRES(Valence >= 2 && Valence <= 4)>
	FPBDSpringConstraintsBase(
		const FSolverParticles& Particles,
		int32 InParticleOffset,
		int32 InParticleCount,
		const TArray<TVector<int32, Valence>>& InConstraints,
		const TConstArrayView<FRealSingle>& StiffnessMultipliers,
		const FSolverVec2& InStiffness,
		bool bTrimKinematicConstraints = false,
		FSolverReal MaxStiffness = FPBDStiffness::DefaultPBDMaxStiffness)
		: Constraints(TrimConstraints(InConstraints, 
			[&Particles, bTrimKinematicConstraints](int32 Index0, int32 Index1)
			{
				return bTrimKinematicConstraints && Particles.InvM(Index0) == (FSolverReal)0. && Particles.InvM(Index1) == (FSolverReal)0.;
			}))
		, ParticleOffset(InParticleOffset)
		, ParticleCount(InParticleCount)
		, Stiffness(
			InStiffness,
			StiffnessMultipliers,
			TConstArrayView<TVec2<int32>>(Constraints),
			ParticleOffset,
			ParticleCount,
			FPBDStiffness::DefaultTableSize,
			FPBDStiffness::DefaultParameterFitBase,
			MaxStiffness)
	{
		// Update distances
		Dists.SetNumUninitialized(Constraints.Num());
		CalculateRestLengths(Particles.XArray(), Dists);
	}

	virtual ~FPBDSpringConstraintsBase() = default;

	// Update stiffness values
	void SetProperties(const FSolverVec2& InStiffness) { Stiffness.SetWeightedValue(InStiffness); }

	// Update stiffness table, as well as the simulation stiffness exponent
	inline void ApplyProperties(const FSolverReal Dt, const int32 NumIterations) { Stiffness.ApplyPBDValues(Dt, NumIterations); }

	const TArray<TVec2<int32>>& GetConstraints() const { return Constraints; }

	void ResetRestLengths(const TConstArrayView<FSolverVec3>& Positions)
	{
		CalculateRestLengths(Positions, TArrayView<FSolverReal>(Dists));
	}

protected:
	template<typename SolverParticlesOrRange>
	FSolverVec3 GetDelta(const SolverParticlesOrRange& Particles, const int32 ConstraintIndex, const FSolverReal ExpStiffnessValue) const
	{
		const auto& Constraint = Constraints[ConstraintIndex];
		const int32 i1 = Constraint[0];
		const int32 i2 = Constraint[1];

		if (Particles.InvM(i2) == (FSolverReal)0. && Particles.InvM(i1) == (FSolverReal)0.)
		{
			return FSolverVec3((FSolverReal)0.);
		}
		const FSolverReal CombinedMass = Particles.InvM(i2) + Particles.InvM(i1);

		const FSolverVec3& P1 = Particles.P(i1);
		const FSolverVec3& P2 = Particles.P(i2);
		FSolverVec3 Direction = P1 - P2;
		const FSolverReal Distance = Direction.SafeNormalize();

		const FSolverVec3 Delta = (Distance - Dists[ConstraintIndex]) * Direction;
		return ExpStiffnessValue * Delta / CombinedMass;
	}

	void CalculateRestLengths(const TConstArrayView<FSolverVec3>& Positions, const TArrayView<FSolverReal>& DistsToCompute) const
	{
		check(DistsToCompute.Num() == Constraints.Num());
		for (int32 Index = 0; Index < Constraints.Num(); ++Index)
		{
			DistsToCompute[Index] = (Positions[Constraints[Index][0]] - Positions[Constraints[Index][1]]).Size();
		}
	}

private:
	template<int32 Valence, typename Predicate UE_REQUIRES(Valence >= 2 && Valence <= 4)>
	TArray<TVector<int32, 2>> TrimConstraints(const TArray<TVector<int32, Valence>>& InConstraints, Predicate TrimPredicate)
	{
		TSet<TVec2<int32>> TrimmedConstraints;
		TrimmedConstraints.Reserve(Valence == 2 ? InConstraints.Num() : InConstraints.Num() * Chaos::Utilities::NChooseR(Valence, 2));

		for (const TVector<int32, Valence>& ConstraintV : InConstraints)
		{
			for (int32 i = 0; i < Valence - 1; ++i)
			{
				for (int32 j = i + 1; j < Valence; ++j)
				{
					const int32 IndexI = ConstraintV[i];
					const int32 IndexJ = ConstraintV[j];

					if (!TrimPredicate(IndexI, IndexJ))
					{
						TrimmedConstraints.Add(IndexI <= IndexJ ? TVec2<int32>(IndexI, IndexJ) : TVec2<int32>(IndexJ, IndexI));
					}
				}
			}
		}
		return TrimmedConstraints.Array();
	}

protected:
	TArray<TVec2<int32>> Constraints;
	TArray<FSolverReal> Dists;
	const int32 ParticleOffset;
	const int32 ParticleCount;
	FPBDStiffness Stiffness;
};

}  // End namespace Chaos::Softs
