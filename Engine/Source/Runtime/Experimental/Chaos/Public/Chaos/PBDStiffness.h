// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/PBDSoftsEvolutionFwd.h"
#include "Chaos/PBDWeightMap.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainersFwd.h"
#include "ChaosStats.h"

DECLARE_CYCLE_STAT(TEXT("Chaos PBD Stiffness Apply Values"), STAT_PBD_StiffnessApplyValues, STATGROUP_Chaos);

namespace Chaos::Softs
{

/**
 * Stiffness class for managing real time update to the weight map and low/high value ranges
 * and to exponentiate the stiffness value depending on the iterations and Dt.
 */
class FPBDStiffness final : public FPBDWeightMap
{
public:
	static constexpr FSolverReal ParameterFrequency = (FSolverReal)120.;  // 60Hz @ 2 iterations as a root for all stiffness values TODO: Make this a global solver parameter
	static constexpr FSolverReal DefaultPBDMaxStiffness = (FSolverReal)1.;
	static constexpr FSolverReal DefaultParameterFitBase = (FSolverReal)1.e3;
	static constexpr int32 DefaultTableSize = 16;

	/**
	 * Weightmap particle constructor. 
	 */
	inline FPBDStiffness(
		const FSolverVec2& InWeightedValue,
		const TConstArrayView<FRealSingle>& Multipliers = TConstArrayView<FRealSingle>(),
		int32 ParticleCount = 0,  // A value of 0 also disables the map
		int32 TableSize = DefaultTableSize,  // Size of the lookup table, can't be more than 256 values, the larger the table the longer it takes to apply changes to the stiffness values
		FSolverReal InParameterFitBase = DefaultParameterFitBase,  // Logarithm base to use in the PBD stiffness parameter fit function
		FSolverReal InMaxStiffness = DefaultPBDMaxStiffness);

	/**
	 * Weightmap constraint constructor. 
	 */
	template<int32 Valence UE_REQUIRES(Valence >= 2 && Valence <= 4)>
	inline FPBDStiffness(
		const FSolverVec2& InWeightedValue,
		const TConstArrayView<FRealSingle>& Multipliers = TConstArrayView<FRealSingle>(),
		const TConstArrayView<TVector<int32, Valence>>& Constraints = TConstArrayView<TVector<int32, Valence>>(),
		int32 ParticleOffset = INDEX_NONE, // Constraints have usually a particle offset added to them compared to the weight maps that always starts at index 0
		int32 ParticleCount = 0,  // A value of 0 also disables the map
		int32 TableSize = DefaultTableSize,  // Size of the lookup table, can't be more than 256 values, the larger the table the longer it takes to apply changes to the stiffness values
		FSolverReal InParameterFitBase = DefaultParameterFitBase,  // Logarithm base to use in the PBD stiffness parameter fit function
		FSolverReal MaxStiffness = DefaultPBDMaxStiffness);

	virtual ~FPBDStiffness() override = default;

	FPBDStiffness(const FPBDStiffness&) = default;
	FPBDStiffness(FPBDStiffness&&) = default;
	FPBDStiffness& operator=(const FPBDStiffness&) = default;
	FPBDStiffness& operator=(FPBDStiffness&&) = default;

	/**
	 * Set the low and high values of the weight map.
	 * The weight map table only gets updated after ApplyValues is called.
	 * Low and high values are clamped between [0, MaxStiffness]
	 */
	void SetWeightedValue(const FSolverVec2& InWeightedValue, FSolverReal MaxStiffness = DefaultPBDMaxStiffness)
	{
		FPBDWeightMap::SetWeightedValue(InWeightedValue.ClampAxes((FSolverReal)0., MaxStiffness));
	}

	/** Update the weight map table with the current simulation parameters. */
	inline void ApplyPBDValues(const FSolverReal Dt, const int32 NumIterations);

	/** Update the weight map table with the current simulation parameters. */
	inline void ApplyXPBDValues(const FSolverReal MaxStiffnesss);

private:
	FSolverReal ParameterFitBase;
	FSolverReal ParameterFitLogBase;
	FSolverReal PrevDtOrMaxStiffness = (FSolverReal)0.;
	int32 PrevNumIterations = 0;
	using FPBDWeightMap::bIsDirty;
};

FPBDStiffness::FPBDStiffness(
	const FSolverVec2& InWeightedValue,
	const TConstArrayView<FRealSingle>& Multipliers,
	int32 ParticleCount,
	int32 TableSize,
	FSolverReal InParameterFitBase,
	FSolverReal MaxStiffness)
	: FPBDWeightMap(FSolverVec2::ZeroVector, Multipliers, ParticleCount, TableSize)
	, ParameterFitBase(InParameterFitBase)
	, ParameterFitLogBase(FMath::Loge(InParameterFitBase))
{
	SetWeightedValue(InWeightedValue, MaxStiffness);
}

template<int32 Valence UE_REQUIRES_DEFINITION(Valence >= 2 && Valence <= 4)>
FPBDStiffness::FPBDStiffness(
	const FSolverVec2& InWeightedValue,
	const TConstArrayView<FRealSingle>& Multipliers,
	const TConstArrayView<TVector<int32, Valence>>& Constraints,
	int32 ParticleOffset,
	int32 ParticleCount,
	int32 TableSize,
	FSolverReal InParameterFitBase,
	FSolverReal MaxStiffness)
	: FPBDWeightMap(FSolverVec2::ZeroVector, Multipliers, Constraints, ParticleOffset, ParticleCount, TableSize)
	, ParameterFitBase(InParameterFitBase)
	, ParameterFitLogBase(FMath::Loge(InParameterFitBase))
{
	SetWeightedValue(InWeightedValue, MaxStiffness);
}

static inline FSolverReal CalcExponentialParameterFit(const FSolverReal ParameterFitBase, const FSolverReal ParameterFitLogBase, const FSolverReal InValue)
{
	// Get a very steep exponential curve between the [0, 1] range to make easier to set the parameter
	// The base has been chosen empirically.
	// ParameterFit = (pow(ParameterFitBase , InValue) - 1) / (ParameterFitBase - 1)
	// Note that ParameterFit = 0 when InValue = 0, and 1 when InValue = 1.
	return (FMath::Exp(ParameterFitLogBase * InValue) - (FSolverReal)1.) / (ParameterFitBase - (FSolverReal)1.);
}

void FPBDStiffness::ApplyPBDValues(const FSolverReal Dt, const int32 NumIterations)
{
	SCOPE_CYCLE_COUNTER(STAT_PBD_StiffnessApplyValues);

	// Calculate the simulation exponent
	const FSolverReal Exponent = Dt * ParameterFrequency / (FSolverReal)NumIterations;

	// Define the stiffness mapping function
	auto SimulationValue = [this, Exponent](const FSolverReal InValue)->FSolverReal
		{
			// If InValue is 1 then LogValue = -inf and the output becomes -inf as well,
			// in order for this function to be continuous, we want the output to be 1 when InValue = 1
			// and need the stiffness to be exactly 0 when the input is 0
			if (InValue <= (FSolverReal)UE_DOUBLE_SMALL_NUMBER)
			{
				return (FSolverReal)0.;
			}
			if (InValue >= (FSolverReal)(1. - UE_DOUBLE_SMALL_NUMBER))
			{
				return (FSolverReal)1.;
			}
			const FSolverReal ParameterFit = CalcExponentialParameterFit(ParameterFitBase, ParameterFitLogBase, InValue);

			// Use simulation dependent stiffness exponent to alleviate the variations in effect when Dt and NumIterations change
			// This is based on the Position-Based Simulation Methods paper (page 8),
			// but uses the delta time in addition of the number of iterations in the calculation of the error term.
			const FSolverReal LogValue = FMath::Loge((FSolverReal)1. - ParameterFit);
			return (FSolverReal)1. - FMath::Exp(LogValue * Exponent);
		};
	
	if (Dt != PrevDtOrMaxStiffness || NumIterations != PrevNumIterations)
	{
		PrevDtOrMaxStiffness = Dt;
		PrevNumIterations = NumIterations;
		bIsDirty = true;
	}

	FPBDWeightMap::ApplyValues(SimulationValue);  // Note, this still needs to be called even when not dirty for dealing with any change in weights
}

void FPBDStiffness::ApplyXPBDValues(const FSolverReal MaxStiffness)
{
	SCOPE_CYCLE_COUNTER(STAT_PBD_StiffnessApplyValues);

	auto SimulationValue = [this, MaxStiffness](const FSolverReal InValue)->FSolverReal
		{
			// Do not apply exponential to XPBDStiffnesses. They are authored in terms of true stiffness values (e.g., kg/s^2), not exponential compliance
			return FMath::Clamp(InValue, (FSolverReal)0., MaxStiffness);
		};

	if (MaxStiffness != PrevDtOrMaxStiffness)
	{
		PrevDtOrMaxStiffness = MaxStiffness;
		bIsDirty = true;
	}

	FPBDWeightMap::ApplyValues(SimulationValue);  // Note, this still needs to be called even when not dirty for dealing with any change in weights
}

}  // End namespace Chaos::Softs
