// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Chaos/PBDSuspensionConstraintTypes.h"
#include "Chaos/ParticleDirtyFlags.h"
#include "Chaos/Framework/PhysicsProxyBase.h"
#include "Chaos/Framework/PhysicsSolverBase.h"
#include "Chaos/PBDConstraintBaseData.h"

namespace Chaos
{

	enum class ESuspensionConstraintFlags : uint64_t
	{												
		Location = 0,
		Enabled = static_cast<uint64_t>(1) << 1,
		Target = static_cast<uint64_t>(1) << 2,
		HardstopStiffness = static_cast<uint64_t>(1) << 3,
		HardstopVelocityCompensation = static_cast<uint64_t>(1) << 4,
		SpringPreload = static_cast<uint64_t>(1) << 5,
		SpringStiffness = static_cast<uint64_t>(1) << 6,
		SpringDamping = static_cast<uint64_t>(1) << 7,
		MinLength = static_cast<uint64_t>(1) << 8,
		MaxLength = static_cast<uint64_t>(1) << 9,
		Axis = static_cast<uint64_t>(1) << 10,
		Normal = static_cast<uint64_t>(1) << 11,

		DummyFlag
	};

	using FSuspensionConstraintDirtyFlags = TDirtyFlags<ESuspensionConstraintFlags>;

	class FSuspensionConstraint : public FConstraintBase
	{
	public:
		CHAOS_API FSuspensionConstraint();
		virtual ~FSuspensionConstraint() override = default;


		CHAOS_API void SetParticleProxy(IPhysicsProxyBase* InParticleProxy);
		CHAOS_API void SetPhysicsBody(FPhysicsObjectHandle& InBody);
		CHAOS_API void SetGroundBody(FPhysicsObjectHandle& InBody);

		const FPBDSuspensionSettings& GetSuspensionSettings() const { return SuspensionSettings.Read(); }

#define CHAOS_INNER_SUSP_PROPERTY(OuterProp, Name, InnerType)\
	void Set##Name(InnerType Val){ OuterProp.Modify(/*bInvalidate=*/true, DirtyFlags, Proxy, [&Val](auto& Data) { Data.Name = Val; }); }\
	InnerType Get##Name() const { return OuterProp.Read().Name;}\

#include "SuspensionProperties.inl"

	protected:
		TChaosProperty<FPBDSuspensionSettings, EChaosProperty::SuspensionSettings> SuspensionSettings;
		TChaosProperty<FSuspensionLocation, EChaosProperty::SuspensionLocation> SuspensionLocation; // location = spring local offset
		TChaosProperty<FProxyBaseProperty, EChaosProperty::SuspensionParticleProxy> SuspensionProxy;
		TChaosProperty<FPhysicsObjectProperty, EChaosProperty::SuspensionPhysicsObject> SuspensionBody;
		TChaosProperty<FPhysicsObjectProperty, EChaosProperty::GroundPhysicsObject> GroundBody;

		virtual void SyncRemoteDataImp(FDirtyPropertiesManager& Manager, int32 DataIdx, FDirtyChaosProperties& RemoteData) override
		{
			SuspensionSettings.SyncRemote(Manager, DataIdx, RemoteData);
			SuspensionLocation.SyncRemote(Manager, DataIdx, RemoteData);
			SuspensionProxy.SyncRemote(Manager, DataIdx, RemoteData);
			SuspensionBody.SyncRemote(Manager, DataIdx, RemoteData);
			GroundBody.SyncRemote(Manager, DataIdx, RemoteData);
		}
	};


} // Chaos
