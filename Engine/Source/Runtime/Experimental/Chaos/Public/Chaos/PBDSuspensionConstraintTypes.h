// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "Chaos/Core.h"
#include "Chaos/Transform.h"
#include "Chaos/Vector.h"

namespace Chaos
{
	class FPBDSuspensionConstraints;

	class FPBDSuspensionConstraintHandle;

	struct FSuspensionLocation
	{
		FVec3 Location = FVec3::ZeroVector;
	};

	class FSingleParticlePhysicsProxy;
	struct FParticleProxyProperty
	{
		FSingleParticlePhysicsProxy* ParticleProxy = nullptr;
	};

	class FPBDSuspensionSettings
	{
	public:

		FPBDSuspensionSettings()
			: Enabled(false)
			, HardstopStiffness(1)
			, HardstopVelocityCompensation(1)
			, SpringPreload(0)
			, SpringStiffness(0)
			, SpringDamping(0)
			, MinLength(0)
			, MaxLength(0)
			, Axis(0,0,1)
			, Target(0, 0, 0)
			, Normal(0, 0, 0)
		{
		}

		FPBDSuspensionSettings(bool InEnabled, FReal InHardstopStiffness, FReal InHardstopVelocityCompensation, FReal InSpringPreload, FReal InSpringStiffness, FReal InDamping, FReal InMinLength, FReal InMaxLength, const FVec3& InAxis, const FVec3& InTarget, const FVec3& InNormal)
			: Enabled(InEnabled)
			, HardstopStiffness(InHardstopStiffness)
			, HardstopVelocityCompensation(InHardstopVelocityCompensation)
			, SpringPreload(InSpringPreload)
			, SpringStiffness(InSpringStiffness)
			, SpringDamping(InDamping)
			, MinLength(InMinLength)
			, MaxLength(InMaxLength)
			, Axis(InAxis)
			, Target(InTarget)
			, Normal(InNormal)
		{
		}

		bool  Enabled;
		FReal HardstopStiffness;
		FReal HardstopVelocityCompensation;
		FReal SpringPreload;
		FReal SpringStiffness;
		FReal SpringDamping;
		FReal MinLength;
		FReal MaxLength;
		FVec3 Axis;
		FVec3 Target;
		FVec3 Normal;
	};

	class FPBDSuspensionResults
	{
	public:
		FPBDSuspensionResults()
			: NetPushOut(FVec3(0))
			, Length(0)
		{
		}

		void Reset()
		{
			*this = FPBDSuspensionResults();
		}

		FVec3 NetPushOut;
		FVec3 HardStopNetPushOut;
		FVec3 HardStopNetImpulse;
		FReal Length;
	};

	class FPBDSuspensionSolverSettings
	{
	public:

		FPBDSuspensionSolverSettings()
		{

		}

	};

}
