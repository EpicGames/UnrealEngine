// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Array.h"
#include "Chaos/Collision/PBDCollisionSolver.h"
#include "Chaos/Evolution/IndexedConstraintContainer.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/PBDSuspensionConstraintTypes.h"
#include "Chaos/PBDSuspensionConstraintData.h"
#include "Chaos/Particle/ParticleUtilities.h"
#include "Chaos/Utilities.h"


namespace Chaos
{
	class FSolverBody;
	class FPBDSuspensionConstraints;

	namespace Private
	{
		class FPBDCollisionSolver;
		class FPBDCollisionSolverManifoldPoint;
	}

	class FPBDSuspensionConstraintHandle final : public TIndexedContainerConstraintHandle<FPBDSuspensionConstraints>
	{
	public:
		using Base = TIndexedContainerConstraintHandle<FPBDSuspensionConstraints>;
		using FConstraintContainer = FPBDSuspensionConstraints;

		FPBDSuspensionConstraintHandle() {}
		FPBDSuspensionConstraintHandle(FConstraintContainer* InConstraintContainer, int32 InConstraintIndex);

		CHAOS_API FPBDSuspensionSettings& GetSettings();
		CHAOS_API const FPBDSuspensionSettings& GetSettings() const;

		CHAOS_API void SetSettings(const FPBDSuspensionSettings& Settings);

		CHAOS_API virtual FParticlePair GetConstrainedParticles() const override final;

		FGeometryParticleHandle* GetVehicleParticle() const { return GetConstrainedParticles()[0]; }
		FGeometryParticleHandle* GetGroundParticle() const { return GetConstrainedParticles()[1]; }

		void SetGroundParticleChanged() { bGroundParticleChanged = true; }
		void ClearGroundParticleChanged() { bGroundParticleChanged = false; }
		bool HasGroundParticleChanged() const
		{
			return bGroundParticleChanged;
		}

		static const FConstraintHandleTypeID& StaticType()
		{
			static FConstraintHandleTypeID STypeID(TEXT("FSuspensionConstraintHandle"), &FIndexedConstraintHandle::StaticType());
			return STypeID;
		}
	protected:
		using Base::ConstraintIndex;
		using Base::ConcreteContainer;

		bool bGroundParticleChanged = false;
	};

	class FPBDSuspensionConstraints : public TPBDIndexedConstraintContainer<FPBDSuspensionConstraints>
	{
	public:
		using Base = TPBDIndexedConstraintContainer<FPBDSuspensionConstraints>;
		using FConstraintContainerHandle = FPBDSuspensionConstraintHandle;
		using FConstraintHandleAllocator = TConstraintHandleAllocator<FPBDSuspensionConstraints>;
		using FHandles = TArray<FConstraintContainerHandle*>;
		using FConstraints = TArrayView<FPBDSuspensionConstraintHandle* const>;

		FPBDSuspensionConstraints(const FPBDSuspensionSolverSettings& InSolverSettings = FPBDSuspensionSolverSettings())
			: TPBDIndexedConstraintContainer<FPBDSuspensionConstraints>(FConstraintContainerHandle::StaticType())
			, SolverSettings(InSolverSettings)
		{}

		FPBDSuspensionConstraints(TArray<FVec3>&& Locations, TArray<TGeometryParticleHandle<FReal,3>*>&& InConstrainedParticles, TArray<FVec3>&& InLocalOffset, TArray<FPBDSuspensionSettings>&& InConstraintSettings)
			: TPBDIndexedConstraintContainer<FPBDSuspensionConstraints>(FConstraintContainerHandle::StaticType())
			, ConstrainedParticles(MoveTemp(InConstrainedParticles)), SuspensionLocalOffset(MoveTemp(InLocalOffset)), ConstraintSettings(MoveTemp(InConstraintSettings))
		{
			if (ConstrainedParticles.Num() > 0)
			{
				Handles.Reserve(ConstrainedParticles.Num());
				for (int32 ConstraintIndex = 0; ConstraintIndex < ConstrainedParticles.Num(); ++ConstraintIndex)
				{
					Handles.Add(HandleAllocator.AllocHandle(this, ConstraintIndex));
				}
			}
		}

		virtual ~FPBDSuspensionConstraints() = default;

		//
		// Constraint Container API
		//

		/**
		 * Get the number of constraints.
		 */
		int32 NumConstraints() const
		{
			return ConstrainedParticles.Num();
		}

		/**
		 * Add a constraint.
		 */
		CHAOS_API FConstraintContainerHandle* AddConstraint(TGeometryParticleHandle<FReal, 3>* Particle, const FVec3& InConstraintFrame, const FPBDSuspensionSettings& InConstraintSettings);

		/**
		 * Remove a constraint.
		 */
		CHAOS_API void RemoveConstraint(int ConstraintIndex);

		FConstraints GetConstraints() { return MakeArrayView(Handles); }

		/*
		* Disconnect the constraints from the attached input particles.
		* This will set the constrained Particle elements to nullptr and
		* set the Enable flag to false.
		*
		* The constraint is unuseable at this point and pending deletion.
		*/

		void DisconnectConstraints(const TSet<TGeometryParticleHandle<FReal, 3>*>& RemovedParticles);

		bool IsConstraintEnabled(int32 ConstraintIndex) const
		{
			return ConstraintEnabledStates[ConstraintIndex];
		}

		void SetConstraintEnabled(int32 ConstraintIndex, bool bEnabled);

		//
		// Constraint API
		//
		const FPBDSuspensionSettings& GetSettings(int32 ConstraintIndex) const
		{
			return ConstraintSettings[ConstraintIndex];
		}

		FPBDSuspensionSettings& GetSettings(int32 ConstraintIndex)
		{
			return ConstraintSettings[ConstraintIndex];
		}

		void SetSettings(int32 ConstraintIndex, const FPBDSuspensionSettings& Settings)
		{
			ConstraintSettings[ConstraintIndex] = Settings;
		}

		void SetTarget(int32 ConstraintIndex, const FVector& TargetPos)
		{
			ConstraintSettings[ConstraintIndex].Target = TargetPos;
		}

		void SetGroundParticle(int32 ConstraintIndex, Chaos::FGeometryParticleHandle* InGroundParticle);

		const FPBDSuspensionResults& GetResults(int32 ConstraintIndex) const
		{
			return ConstraintResults[ConstraintIndex];
		}

		FHandles& GetConstraintHandles()
		{
			return Handles;
		}
		const FHandles& GetConstConstraintHandles() const
		{
			return Handles;
		}

		const FConstraintContainerHandle* GetConstraintHandle(int32 ConstraintIndex) const
		{
			return Handles[ConstraintIndex];
		}

		FConstraintContainerHandle* GetConstraintHandle(int32 ConstraintIndex)
		{
			return Handles[ConstraintIndex];
		}

		/**
		 * Get the particles that are affected by the specified constraint.
		 */
		TVec2<TGeometryParticleHandle<FReal, 3>*> GetConstrainedParticles(int32 ConstraintIndex) const
		{
			check(ConstraintIndex < ConstrainedParticles.Num());
			check(ConstraintIndex < GroundParticles.Num());
			check(ConstrainedParticles.Num() == GroundParticles.Num());

			return { ConstrainedParticles[ConstraintIndex], GroundParticles[ConstraintIndex] };
		}

		/**
		 * Get the world-space constraint positions for each body.
		 */
		const FVec3& GetConstraintPosition(int ConstraintIndex) const
		{
			return SuspensionLocalOffset[ConstraintIndex];
		}

		void SetConstraintPosition(const int32 ConstraintIndex, const FVec3& Position)
		{
			SuspensionLocalOffset[ConstraintIndex] = Position;
		}

		//
		// FConstraintContainer Implementation
		//
		virtual int32 GetNumConstraints() const override final { return NumConstraints(); }
		virtual void ResetConstraints() override final {}
		CHAOS_API virtual void AddConstraintsToGraph(Private::FPBDIslandManager& IslandManager) override final;
		virtual void PrepareTick() override final {}
		virtual void UnprepareTick() override final {}

		//
		// TSimpleConstraintContainerSolver API - used by RBAN
		//
		CHAOS_API void AddBodies(FSolverBodyContainer& SolverBodyContainer);
		CHAOS_API void GatherInput(const FReal Dt);
		CHAOS_API void ScatterOutput(const FReal Dt);
		CHAOS_API void ApplyPositionConstraints(const FReal Dt, const int32 It, const int32 NumIts);
		CHAOS_API void ApplyVelocityConstraints(const FReal Dt, const int32 It, const int32 NumIts);
		void ApplyProjectionConstraints(const FReal Dt, const int32 It, const int32 NumIts) {}

		//
		// TIndexedConstraintContainerSolver API - used by World solvers
		//
		CHAOS_API void AddBodies(const TArrayView<int32>& ConstraintIndices, FSolverBodyContainer& SolverBodyContainer);
		CHAOS_API void GatherInput(const TArrayView<int32>& ConstraintIndices, const FReal Dt);
		CHAOS_API void ScatterOutput(const TArrayView<int32>& ConstraintIndices, const FReal Dt);
		CHAOS_API void ApplyPositionConstraints(const TArrayView<int32>& ConstraintIndices, const FReal Dt, const int32 It, const int32 NumIts);
		CHAOS_API void ApplyVelocityConstraints(const TArrayView<int32>& ConstraintIndices, const FReal Dt, const int32 It, const int32 NumIts);
		void ApplyProjectionConstraints(const TArrayView<int32>& ConstraintIndices, const FReal Dt, const int32 It, const int32 NumIts) {}


	protected:
		using Base::GetConstraintIndex;
		using Base::SetConstraintIndex;

	private:
		CHAOS_API void AddBodies(const int32 ConstraintIndex, FSolverBodyContainer& SolverBodyContainer);
		CHAOS_API void GatherInput(const int32 ConstraintIndex, FReal Dt);
		CHAOS_API void ScatterOutput(const int32 ConstraintIndex, FReal Dt);
		CHAOS_API void ApplyPositionConstraint(const int32 ConstraintIndex, const FReal Dt, const int32 It, const int32 NumIts);
		CHAOS_API void ApplyVelocityConstraint(const int32 ConstraintIndex, const FReal Dt, const int32 It, const int32 NumIts);
		CHAOS_API void ApplySingle(int32 ConstraintIndex, const FReal Dt);
		
		FPBDSuspensionSolverSettings SolverSettings;

		TArray<FGeometryParticleHandle*> ConstrainedParticles;
		TArray<FGeometryParticleHandle*> GroundParticles;
		TArray<FVec3> SuspensionLocalOffset;
		TArray<FPBDSuspensionSettings> ConstraintSettings;
		TArray<FPBDSuspensionResults> ConstraintResults;
		TArray<bool> ConstraintEnabledStates;

		TArray<FSolverBodyPtrPair> ConstraintSolverBodies;

		FHandles Handles;
		FConstraintHandleAllocator HandleAllocator;

		TArray<Private::FPBDCollisionSolver> CollisionSolvers;
		TArray<Private::FPBDCollisionSolverManifoldPoint> CollisionSolverManifoldPoints;
	};
}
