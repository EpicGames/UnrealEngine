// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// HEADER_UNIT_SKIP - Internal

#include "Chaos/PBDTetConstraintsBase.h"

namespace Chaos::Softs
{

class FPBDTetConstraints : public FPBDTetConstraintsBase
{
	typedef FPBDTetConstraintsBase Base;
	using Base::Constraints;

  public:
	  FPBDTetConstraints(const FSolverParticles& InParticles, TArray<TVec4<int32>>&& InConstraints, const FSolverReal InStiffness = (FSolverReal)1)
	    : Base(InParticles, MoveTemp(InConstraints), InStiffness) {}
	virtual ~FPBDTetConstraints() override = default;

	void Apply(FSolverParticles& InParticles, const FReal dt) const
	{
		for (int i = 0; i < Constraints.Num(); ++i)
		{
			const TVec4<int32>& Constraint = Constraints[i];
			const int32 i1 = Constraint[0];
			const int32 i2 = Constraint[1];
			const int32 i3 = Constraint[2];
			const int32 i4 = Constraint[3];
			const TVec4<FSolverVec3> Grads = Base::GetGradients(InParticles, i);
			const FSolverReal S = Base::GetScalingFactor(InParticles, i, Grads);
			InParticles.P(i1) -= S * InParticles.InvM(i1) * Grads[0];
			InParticles.P(i2) -= S * InParticles.InvM(i2) * Grads[1];
			InParticles.P(i3) -= S * InParticles.InvM(i3) * Grads[2];
			InParticles.P(i4) -= S * InParticles.InvM(i4) * Grads[3];
		}
	}
};

}  // End namespace Chaos::Softs
