// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Array.h"
#include "Chaos/PBDSoftsEvolutionFwd.h"
#include "Chaos/PBDParticles.h"
#include "Chaos/DynamicParticles.h"
#include "Chaos/PBDParticles.h"
#include "Chaos/PBDSoftsSolverParticles.h"

namespace Chaos::Softs
{

class FPBDTetConstraintsBase
{
public:
	FPBDTetConstraintsBase(const FSolverParticles& InParticles, TArray<TVec4<int32>>&& InConstraints, const FSolverReal InStiffness = (FSolverReal)1.)
	    : Constraints(InConstraints), Stiffness(InStiffness)
	{
		for (const TVec4<int32>& Constraint : Constraints)
		{
			const FSolverVec3& P1 = InParticles.GetX(Constraint[0]);
			const FSolverVec3& P2 = InParticles.GetX(Constraint[1]);
			const FSolverVec3& P3 = InParticles.GetX(Constraint[2]);
			const FSolverVec3& P4 = InParticles.GetX(Constraint[3]);
			Volumes.Add(FSolverVec3::DotProduct(FSolverVec3::CrossProduct(P2 - P1, P3 - P1), P4 - P1) / (FSolverReal)6.);
		}
	}
	virtual ~FPBDTetConstraintsBase() = default;

	TVec4<FSolverVec3> GetGradients(const FSolverParticles& InParticles, const int32 i) const
	{
		TVec4<FSolverVec3> Grads;
		const TVec4<int32>& Constraint = Constraints[i];
		const FSolverVec3& P1 = InParticles.P(Constraint[0]);
		const FSolverVec3& P2 = InParticles.P(Constraint[1]);
		const FSolverVec3& P3 = InParticles.P(Constraint[2]);
		const FSolverVec3& P4 = InParticles.P(Constraint[3]);
		const FSolverVec3 P2P1 = P2 - P1;
		const FSolverVec3 P4P1 = P4 - P1;
		const FSolverVec3 P3P1 = P3 - P1;
		Grads[1] = FSolverVec3::CrossProduct(P3P1, P4P1) / (FSolverReal)6.;
		Grads[2] = FSolverVec3::CrossProduct(P4P1, P2P1) / (FSolverReal)6.;
		Grads[3] = FSolverVec3::CrossProduct(P2P1, P3P1) / (FSolverReal)6.;
		Grads[0] = -(Grads[1] + Grads[2] + Grads[3]);
		return Grads;
	}

	FSolverReal GetScalingFactor(const FSolverParticles& InParticles, const int32 i, const TVec4<FSolverVec3>& Grads) const
	{
		const TVec4<int32>& Constraint = Constraints[i];
		const int32 i1 = Constraint[0];
		const int32 i2 = Constraint[1];
		const int32 i3 = Constraint[2];
		const int32 i4 = Constraint[3];
		const FSolverVec3& P1 = InParticles.P(i1);
		const FSolverVec3& P2 = InParticles.P(i2);
		const FSolverVec3& P3 = InParticles.P(i3);
		const FSolverVec3& P4 = InParticles.P(i4);
		const FSolverReal Volume = FSolverVec3::DotProduct(FSolverVec3::CrossProduct(P2 - P1, P3 - P1), P4 - P1) / (FSolverReal)6.;
		const FSolverReal S = (Volume - Volumes[i]) / (
			InParticles.InvM(i1) * Grads[0].SizeSquared() +
			InParticles.InvM(i2) * Grads[1].SizeSquared() + 
			InParticles.InvM(i3) * Grads[2].SizeSquared() + 
			InParticles.InvM(i4) * Grads[3].SizeSquared());
		return Stiffness * S;
	}

protected:
	TArray<TVec4<int32>> Constraints;

private:
	TArray<FSolverReal> Volumes;
	FSolverReal Stiffness;
};

}  // End namespace Chaos::Softs
