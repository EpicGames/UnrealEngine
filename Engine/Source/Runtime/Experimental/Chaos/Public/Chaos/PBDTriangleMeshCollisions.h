// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/PBDFlatWeightMap.h"
#include "Chaos/PBDSoftsEvolutionFwd.h"
#include "Chaos/TriangleMesh.h"
#include "Chaos/HierarchicalSpatialHash.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "Templates/PimplPtr.h"

// This does initialization for PBDCollisionSpringConstraints and PBDTriangleMeshIntersections, 
// including intersection detection and global intersection analysis
//
namespace Chaos::Softs
{
class FPBDTriangleMeshCollisions
{
public:
	struct FContourMinimizationIntersection
	{
		TVec2<int32> EdgeVertices;
		TVec3<int32> FaceVertices;
		FSolverVec3 LocalGradientVector;
		FSolverVec3 GlobalGradientVector;
	};
	
	// Vertices (and Triangles) are assigned FGIAColors by flood-filling global intersection contours.
	// ContourIndex represents which pair of global contours, up to 31 unique contour pairs--then the contour index will reused. 
	// In practice this seems to be enough unique contours. It's OK if multiple contours have the same index as long
	// as their intersecting regions don't interact, which is unlikely.
	// Contours come in pairs representing two regions intersecting (except for "loop" contours where a region of cloth intersects itself--think pinch regions). 
	// These are "Colors" represented by setting the ColorBit to 0 or 1 corresponding with the given contour index.
	struct FGIAColor
	{
		int32 ContourIndexBits = 0;
		int32 ColorBits = 0;

		static constexpr int32 LoopContourIndex = 0;
		static constexpr int32 LoopBits = 1 << LoopContourIndex;
		static constexpr int32 NonLoopMask = ~LoopBits;
		static constexpr int32 BoundaryContourIndex = 0; // Same as Loop. Loop is ColorA, Boundary is ColorB
		
		void SetContourColor(int32 ContourIndex, bool bIsColorB)
		{
			check(ContourIndex < 32);
			ContourIndexBits |= 1 << ContourIndex;
			if (bIsColorB)
			{
				ColorBits |= 1 << ContourIndex;
			}
			else
			{
				ColorBits &= ~(1 << ContourIndex);
			}
		}

		bool HasContourColorSet(int32 ContourIndex) const
		{
			check(ContourIndex < 32);
			return ContourIndexBits & (1 << ContourIndex);
		}

		bool IsLoop() const
		{
			return (ContourIndexBits & LoopBits) && (ColorBits & LoopBits);
		}

		void SetLoop()
		{
			SetContourColor(LoopContourIndex, true);
		}

		void SetBoundary()
		{
			SetContourColor(BoundaryContourIndex, false);
		}

		bool IsBoundary() const
		{
			return (ContourIndexBits & LoopBits) && !(ColorBits & LoopBits);
		}

		// Because they are opposite colors with a shared contour index. This will cause repulsion forces to attract and fix the intersection.
		// NOTE: We flip normals if ANY of the TriVertColors are opposite the PointColor or if the TriColor (used for thin regions is flipped). This does a better job for thin features
		// than only flipping normals if ALL TriVertColors are opposite.
		static bool ShouldFlipNormal(const FGIAColor& Color0, const FGIAColor& Color1)
		{
			const int32 SharedContourBits = Color0.ContourIndexBits & Color1.ContourIndexBits & NonLoopMask;
			const int32 FlippedColorBits = Color0.ColorBits ^ Color1.ColorBits;
			return FlippedColorBits & SharedContourBits;
		}
	};

	// Debug display of intersection contours
	struct FBarycentricPoint
	{
		FSolverVec2 Bary;
		TVec3<int32> Vertices;
	};

	enum struct FContourType : int8
	{
		Open = 0,
		Loop,
		BoundaryClosed,
		BoundaryOpen,
		Contour0,
		Contour1,
		Count
	};

	class FTriangleSubMesh
	{
	public:
		FTriangleSubMesh(const FTriangleMesh& InFullMesh)
			: FullMesh(InFullMesh)
		{}

		template<typename SolverParticlesOrRange>
		void Init(const SolverParticlesOrRange& Particles, const TSet<int32>& DisabledFaces, bool bCollideAgainstAllKinematicVertices, const TSet<int32>& EnabledKinematicFaces, const bool bOnlyCollideKinematics = false);

		void InitAllDynamic();

		const FTriangleMesh& GetFullMesh() const 
		{
			return FullMesh;
		}

		const FTriangleMesh& GetDynamicSubMesh() const
		{
			return DynamicSubMesh;
		}

		const FTriangleMesh& GetKinematicColliderSubMesh() const
		{
			return KinematicColliderSubMesh;
		}

		bool IsElementDynamic(int32 FullMeshIndex) const
		{
			return FullMeshToSubMeshIndices[FullMeshIndex].SubMeshType == ESubMeshType::Dynamic;
		}

		bool IsElementKinematicCollider(int32 FullMeshIndex) const
		{
			return FullMeshToSubMeshIndices[FullMeshIndex].SubMeshType == ESubMeshType::Kinematic;
		}

		int32 GetSubMeshElementIndex(int32 FullMeshIndex) const
		{
			check(FullMeshToSubMeshIndices[FullMeshIndex].SubMeshType != ESubMeshType::Invalid);
			return FullMeshToSubMeshIndices[FullMeshIndex].SubMeshIndex;
		}

		int32 GetFullMeshElementIndexFromDynamicElement(int32 DynamicMeshIndex) const
		{
			return DynamicSubMeshToFullMeshIndices[DynamicMeshIndex];
		}

		int32 GetFullMeshElementIndexFromKinematicElement(int32 KinematicMeshIndex) const
		{
			return KinematicColliderSubMeshToFullMeshIndices[KinematicMeshIndex];
		}

		const TArray<int32>& GetDynamicVertices() const { return DynamicVertices; }


	private:
		const FTriangleMesh& FullMesh;
		FTriangleMesh DynamicSubMesh;
		FTriangleMesh KinematicColliderSubMesh;
		
		enum struct ESubMeshType : int32
		{
			Invalid = 0,
			Dynamic = 1,
			Kinematic = 2
		};
		struct FFullToSubMeshIndex
		{
			int32 SubMeshIndex : 30;
			ESubMeshType SubMeshType : 2;
		};
		TArray<FFullToSubMeshIndex> FullMeshToSubMeshIndices;
		TArray<int32> DynamicSubMeshToFullMeshIndices;
		TArray<int32> KinematicColliderSubMeshToFullMeshIndices;
		TArray<int32> DynamicVertices; // Will be empty if InitAllDynamic was used to initialize
	};

	static bool IsEnabled(const FCollectionPropertyConstFacade& PropertyCollection)
	{
		return GetUseSelfCollisions(PropertyCollection, false);
	}

	FPBDTriangleMeshCollisions(
		const int32 InOffset,
		const int32 InNumParticles,
		const TMap<FString, const TSet<int32>*>& FaceSets,
		const FTriangleMesh& InTriangleMesh,
		const FCollectionPropertyConstFacade& PropertyCollection
	)
		: CollidableSubMesh(InTriangleMesh)
		, Offset(InOffset)
		, NumParticles(InNumParticles)
		, bUseSelfIntersections(GetUseSelfIntersections(PropertyCollection, false))
		, bGlobalIntersectionAnalysis(GetUseSelfIntersections(PropertyCollection, false) && GetUseGlobalIntersectionAnalysis(PropertyCollection, true))
		, bContourMinimization(GetUseSelfIntersections(PropertyCollection, false) && GetUseContourMinimization(PropertyCollection, true))
		, NumContourMinimizationPostSteps(GetUseSelfIntersections(PropertyCollection, false) ? GetNumContourMinimizationPostSteps(PropertyCollection, 0) : 0)
		, bUseGlobalPostStepContours(GetUseGlobalPostStepContours(PropertyCollection, true))
		, bOnlyCollideWithKinematics(GetSelfCollideAgainstKinematicCollidersOnly(PropertyCollection, false))
		, bSelfCollideAgainstAllKinematicVertices(GetSelfCollideAgainstAllKinematicVertices(PropertyCollection, false))
		, bCollidableSubMeshDirty(true)
		, UseSelfIntersectionsIndex(PropertyCollection)
		, UseGlobalIntersectionAnalysisIndex(PropertyCollection)
		, UseContourMinimizationIndex(PropertyCollection)
		, NumContourMinimizationPostStepsIndex(PropertyCollection)
		, UseGlobalPostStepContoursIndex(PropertyCollection)
		, SelfCollideAgainstKinematicCollidersOnlyIndex(PropertyCollection)
		, SelfCollideAgainstAllKinematicVerticesIndex(PropertyCollection)
		, SelfCollisionDisabledFacesIndex(PropertyCollection)
		, SelfCollisionEnabledKinematicFacesIndex(PropertyCollection)
	{
		if (const TSet<int32>* const InDisabledFaces = FaceSets.FindRef(GetSelfCollisionDisabledFacesString(PropertyCollection, SelfCollisionDisabledFacesName.ToString()), nullptr))
		{
			DisabledFaces = *InDisabledFaces;
		}	
	}

	UE_DEPRECATED(5.4, "Use Constructor with FaceSets")
	FPBDTriangleMeshCollisions(
		const int32 InOffset,
		const int32 InNumParticles,
		const FTriangleMesh& InTriangleMesh,
		const FCollectionPropertyConstFacade& PropertyCollection)
		: FPBDTriangleMeshCollisions(InOffset, InNumParticles, TMap<FString, const TSet<int32>*>(), InTriangleMesh, PropertyCollection)
	{}

	FPBDTriangleMeshCollisions(
		const int32 InOffset,
		const int32 InNumParticles,
		const FTriangleMesh& InTriangleMesh,
		bool bInGlobalIntersectionAnalysis,
		bool bInContourMinimization
	)
		:CollidableSubMesh(InTriangleMesh)
		, Offset(InOffset)
		, NumParticles(InNumParticles)
		, bUseSelfIntersections(bInGlobalIntersectionAnalysis || bInContourMinimization)
		, bGlobalIntersectionAnalysis(bInGlobalIntersectionAnalysis)
		, bContourMinimization(bInContourMinimization)
		, bOnlyCollideWithKinematics(false)
		, bSelfCollideAgainstAllKinematicVertices(false)
		, bCollidableSubMeshDirty(true)
		, UseSelfIntersectionsIndex(ForceInit)
		, UseGlobalIntersectionAnalysisIndex(ForceInit)
		, UseContourMinimizationIndex(ForceInit)
		, NumContourMinimizationPostStepsIndex(ForceInit)
		, UseGlobalPostStepContoursIndex(ForceInit)
		, SelfCollideAgainstKinematicCollidersOnlyIndex(ForceInit)
		, SelfCollideAgainstAllKinematicVerticesIndex(ForceInit)
		, SelfCollisionDisabledFacesIndex(ForceInit)
		, SelfCollisionEnabledKinematicFacesIndex(ForceInit)
	{}

	virtual ~FPBDTriangleMeshCollisions() = default;

	void SetProperties(const FCollectionPropertyConstFacade& PropertyCollection,
		const TMap<FString, const TSet<int32>*>& FaceSets)
	{
		const bool bSelfIntersectionsMutable = IsUseSelfIntersectionsMutable(PropertyCollection);
		if (bSelfIntersectionsMutable)
		{
			bUseSelfIntersections = GetUseSelfIntersections(PropertyCollection);
		}
		if (bUseSelfIntersections)
		{
			if (bSelfIntersectionsMutable || IsUseGlobalIntersectionAnalysisMutable(PropertyCollection))
			{
				bGlobalIntersectionAnalysis = GetUseGlobalIntersectionAnalysis(PropertyCollection);
			}
			if (bSelfIntersectionsMutable || IsUseContourMinimizationMutable(PropertyCollection))
			{
				bContourMinimization = GetUseContourMinimization(PropertyCollection);
			}
			if (bSelfIntersectionsMutable || IsNumContourMinimizationPostStepsMutable(PropertyCollection))
			{
				NumContourMinimizationPostSteps = GetNumContourMinimizationPostSteps(PropertyCollection);
			}
			if (bSelfIntersectionsMutable || IsUseGlobalPostStepContoursMutable(PropertyCollection))
			{
				bUseGlobalPostStepContours = GetUseGlobalPostStepContours(PropertyCollection);
			}
		}
		else
		{
			bGlobalIntersectionAnalysis = bContourMinimization = false;
			NumContourMinimizationPostSteps = 0;
		}

		if (IsSelfCollideAgainstKinematicCollidersOnlyMutable(PropertyCollection))
		{
			const bool bNewValue = GetSelfCollideAgainstKinematicCollidersOnly(PropertyCollection);
			if (bNewValue != bOnlyCollideWithKinematics)
			{
				bOnlyCollideWithKinematics = bNewValue;
				bCollidableSubMeshDirty = true;
			}
		}

		if (IsSelfCollideAgainstAllKinematicVerticesMutable(PropertyCollection))
		{
			const bool bNewValue = GetSelfCollideAgainstAllKinematicVertices(PropertyCollection);
			if (bNewValue != bSelfCollideAgainstAllKinematicVertices)
			{
				bSelfCollideAgainstAllKinematicVertices = bNewValue;
				bCollidableSubMeshDirty = true;
			}
		}
		if (IsSelfCollisionDisabledFacesMutable(PropertyCollection) && IsSelfCollisionDisabledFacesStringDirty(PropertyCollection))
		{
			if (const TSet<int32>* const InDisabledFaces = FaceSets.FindRef(GetSelfCollisionDisabledFacesString(PropertyCollection, SelfCollisionDisabledFacesName.ToString()), nullptr))
			{
				DisabledFaces = *InDisabledFaces;
				bCollidableSubMeshDirty = true;
			}
			else
			{
				if (!DisabledFaces.IsEmpty())
				{
					DisabledFaces.Reset();
					bCollidableSubMeshDirty = true;
				}
			}
		}
		if (IsSelfCollisionEnabledKinematicFacesMutable(PropertyCollection) && IsSelfCollisionEnabledKinematicFacesStringDirty(PropertyCollection))
		{
			if (const TSet<int32>* const InEnabledFaces = FaceSets.FindRef(GetSelfCollisionEnabledKinematicFacesString(PropertyCollection, SelfCollisionEnabledKinematicFacesName.ToString()), nullptr))
			{
				EnabledKinematicFaces = *InEnabledFaces;
				bCollidableSubMeshDirty = true;
			}
			else
			{
				if (!EnabledKinematicFaces.IsEmpty())
				{
					EnabledKinematicFaces.Reset();
					bCollidableSubMeshDirty = true;
				}
			}
		}
	}

	UE_DEPRECATED(5.4, "Use version with FaceSets")
	void SetProperties(const FCollectionPropertyConstFacade& PropertyCollection)
	{
		SetProperties(PropertyCollection, TMap<FString, const TSet<int32>*>());
	}

	template<typename SolverParticlesOrRange>
	void Init(const SolverParticlesOrRange& Particles, const FSolverReal MinProximityQueryRadius = (FSolverReal)0.)
	{
		Init(Particles, FPBDFlatWeightMap(FSolverVec2(MinProximityQueryRadius * (FSolverReal).5f)));
	}

	template<typename SolverParticlesOrRange>
	CHAOS_API void Init(const SolverParticlesOrRange& Particles, const FPBDFlatWeightMap& ThicknessMap);

	template<typename SolverParticlesOrRange>
	void InitFlesh(const SolverParticlesOrRange& Particles, const FSolverReal MinProximityQueryRadius = (FSolverReal)0., const bool bUseFullMesh = false)
	{
		InitFlesh(Particles, FPBDFlatWeightMap(FSolverVec2(MinProximityQueryRadius * (FSolverReal).5f)), bUseFullMesh);
	}

	template<typename SolverParticlesOrRange>
	CHAOS_API void InitFlesh(const SolverParticlesOrRange& Particles, const FPBDFlatWeightMap& ThicknessMap, const bool bUseFullMesh = false);

	template<typename SolverParticlesOrRange>
	CHAOS_API void PostStepInit(const SolverParticlesOrRange& Particles);

	void SetGlobalIntersectionAnalysis(bool bInGlobalIntersectionAnalysis) { bGlobalIntersectionAnalysis = bInGlobalIntersectionAnalysis; }
	void SetContourMinimization(bool bInContourMinimization) { bContourMinimization = bInContourMinimization; }

	int32 GetNumContourMinimizationPostSteps() const
	{
		return NumContourMinimizationPostSteps;
	}

	bool GetCollidableSubMeshDirty() const { return bCollidableSubMeshDirty; }

	const FTriangleSubMesh& GetCollidableSubMesh() const { return CollidableSubMesh; }
	UE_DEPRECATED(5.4, "Use GetDynamicSpatialHash or GetKinematicColliderSpatialHash")
	const FTriangleMesh::TSpatialHashType<FSolverReal>& GetSpatialHash() const { return DynamicSubMeshSpatialHash; }
	const FTriangleMesh::TSpatialHashType<FSolverReal>& GetDynamicSpatialHash() const { return DynamicSubMeshSpatialHash; }
	const FTriangleMesh::TSpatialHashType<FSolverReal>& GetKinematicColliderSpatialHash() const { return KinematicSubMeshSpatialHash; }
	const TArray<FContourMinimizationIntersection>& GetContourMinimizationIntersections() const { return ContourMinimizationIntersections; }
	const TConstArrayView<FGIAColor> GetVertexGIAColors() const { return bGlobalIntersectionAnalysis && VertexGIAColors.Num() == NumParticles ? TConstArrayView<FGIAColor>(VertexGIAColors.GetData() - Offset, NumParticles + Offset) : TConstArrayView<FGIAColor>(); }
	const TArray<FGIAColor>& GetTriangleGIAColors() const { return TriangleGIAColors; }
	const TArray<TArray<FBarycentricPoint>>& GetIntersectionContourPoints() const { return IntersectionContourPoints; }
	const TArray<FContourType>& GetIntersectionContourTypes() const { return IntersectionContourTypes; }

	// Same data but for the post step contour minimization. Just making them separate arrays
	// for debug draw purposes.
	const TArray<FContourMinimizationIntersection>& GetPostStepContourMinimizationIntersections() const { return PostStepContourMinimizationIntersections; }
	const TArray<TArray<FBarycentricPoint>>& GetPostStepIntersectionContourPoints() const { return PostStepIntersectionContourPoints; }


	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(UseSelfIntersections, bool);
private:

	FTriangleSubMesh CollidableSubMesh;
	int32 Offset;
	int32 NumParticles;
	bool bUseSelfIntersections;
	bool bGlobalIntersectionAnalysis;
	bool bContourMinimization;	
	int32 NumContourMinimizationPostSteps = 0;
	bool bUseGlobalPostStepContours = true;
	bool bOnlyCollideWithKinematics;
	bool bSelfCollideAgainstAllKinematicVertices;
	TSet<int32> DisabledFaces;
	TSet<int32> EnabledKinematicFaces;

	bool bCollidableSubMeshDirty = true;

	
	FTriangleMesh::TSpatialHashType<FSolverReal> DynamicSubMeshSpatialHash;
	FTriangleMesh::TSpatialHashType<FSolverReal> KinematicSubMeshSpatialHash;
	TArray<FContourMinimizationIntersection> ContourMinimizationIntersections;
	TArray<FGIAColor> VertexGIAColors;
	TArray<FGIAColor> TriangleGIAColors;

	// Scratch buffers used by Init and PostInit. They live here so they can be reused rather than reallocated.
	struct FScratchBuffers;
	TPimplPtr<FScratchBuffers> ScratchBuffers;

	// Debug display of intersection contours
	TArray<TArray<FBarycentricPoint>> IntersectionContourPoints;
	TArray<FContourType> IntersectionContourTypes;

	// PostStep contour data. Keeping it separate for debug drawing for now.
	TArray<FContourMinimizationIntersection> PostStepContourMinimizationIntersections;
	TArray<TArray<FBarycentricPoint>> PostStepIntersectionContourPoints;

	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(UseSelfCollisions, bool);
	UE_CHAOS_DECLARE_INDEXED_PROPERTYCOLLECTION_NAME(UseSelfIntersections, bool);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(UseGlobalIntersectionAnalysis, bool);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(UseContourMinimization, bool);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(NumContourMinimizationPostSteps, int32);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(UseGlobalPostStepContours, bool);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(SelfCollideAgainstKinematicCollidersOnly, bool);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(SelfCollideAgainstAllKinematicVertices, bool);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(SelfCollisionDisabledFaces, bool);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(SelfCollisionEnabledKinematicFaces, bool);
};

}  // End namespace Chaos::Softs
