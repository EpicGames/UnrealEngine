// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/PBDSoftsEvolutionFwd.h"
#include "Chaos/PBDTriangleMeshCollisions.h"
#include "Chaos/CollectionPropertyFacade.h"

// Resolve intersections via contour minimization. Contour Minimization data is calculated in PBDTriangleMeshCollisions.
namespace Chaos::Softs
{
class FPBDTriangleMeshIntersections
{
public:
	static bool IsEnabled(const FCollectionPropertyConstFacade& PropertyCollection)
	{
		return FPBDTriangleMeshCollisions::IsEnabled(PropertyCollection);
	}

	FPBDTriangleMeshIntersections(
		const int32 InOffset,
		const int32 InNumParticles,
		const FTriangleMesh& InTriangleMesh
	)
		:TriangleMesh(InTriangleMesh)
		, Offset(InOffset)
		, NumParticles(InNumParticles)
	{}

	~FPBDTriangleMeshIntersections() = default;

	template<typename SolverParticlesOrRange>
	CHAOS_API void Apply(SolverParticlesOrRange& Particles, const TArray<FPBDTriangleMeshCollisions::FContourMinimizationIntersection>& Intersections, const FSolverReal Dt) const;

private:
	const FTriangleMesh& TriangleMesh;
	int32 Offset;
	int32 NumParticles;
};

}  // End namespace Chaos::Softs
