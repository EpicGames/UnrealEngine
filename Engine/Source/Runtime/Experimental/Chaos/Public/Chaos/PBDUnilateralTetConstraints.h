// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/CollectionEmbeddedSpringConstraintFacade.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "Chaos/Core.h"
#include "Chaos/PBDFlatWeightMap.h"
#include "Chaos/PBDSoftsEvolutionFwd.h"
#include "Chaos/PBDSoftsSolverParticles.h"
#include "Chaos/SoftsSolverParticlesRange.h"
#include "Containers/Array.h"

namespace Chaos::Softs
{

	class FPBDUnilateralTetConstraints
	{
	public:
		static constexpr FSolverReal MinStiffness = (FSolverReal)0;
		static constexpr FSolverReal MaxStiffness = (FSolverReal)1.f;
		static constexpr FSolverReal DefaultStiffness = (FSolverReal)0.5f;

		CHAOS_API FPBDUnilateralTetConstraints(
			const FSolverParticlesRange& Particles,
			TArray<TVector<int32, 4>>&& InConstraints,
			TArray<FSolverReal>&& InVolumes,
			FSolverReal InStiffness,
			int32 InMaxNumIters);

		~FPBDUnilateralTetConstraints() = default;

		void ApplyProperties(const FSolverReal /*Dt*/, const int32 /*NumIterations*/)
		{
		}

		CHAOS_API void Apply(FSolverParticlesRange& Particles, const FSolverReal Dt) const;
		
		const TArray<TVector<int32, 4>>& GetConstraints() const { return Constraints; }

#if CHAOS_DEBUG_DRAW
		const TArray<bool>& GetConstraintIsActive() const { return ConstraintIsActive; }
#endif

	private:

		void TrimKinematicConstraints(const FSolverParticlesRange& Particles);
		void InitColor(const FSolverParticlesRange& Particles);

		void ApplyVolumeConstraint(FSolverParticlesRange& Particles, const FSolverReal Dt) const;
#if INTEL_ISPC
		void ApplyVolumeConstraint_ISPC(FSolverParticlesRange& Particles, const FSolverReal Dt) const {}
#endif

		TArray<TVector<int32, 4>> Constraints;
		TArray<FSolverReal> Volumes;

		TArray<int32> ConstraintsPerColorStartIndex; // Constraints are ordered so each batch is contiguous. This is ColorNum + 1 length so it can be used as start and end.

#if CHAOS_DEBUG_DRAW
		// Used for debug draw to show which constraints were active last apply
		mutable TArray<bool> ConstraintIsActive;
#endif

	protected:
		FSolverReal Stiffness;
		int32 MaxNumIters;
	};

	class FPBDVertexFaceRepulsionConstraints final : public FPBDUnilateralTetConstraints
	{
	public:

		static bool IsEnabled(const FCollectionPropertyConstFacade& PropertyCollection)
		{
			return IsVertexFaceRepulsionStiffnessEnabled(PropertyCollection, false);
		}

		CHAOS_API FPBDVertexFaceRepulsionConstraints(
			const FSolverParticlesRange& Particles,
			const FCollectionPropertyConstFacade& PropertyCollection,
			const FEmbeddedSpringConstraintFacade& SpringConstraintFacade);

		CHAOS_API void SetProperties(const FCollectionPropertyConstFacade& PropertyCollection);

	private:
		static TArray<TVector<int32, 4>> ExtractConstraintIndices(const FEmbeddedSpringConstraintFacade& SpringConstraintFacade);
		static TArray<FSolverReal> ExtractVolumes(const FSolverParticlesRange& Particles, const FEmbeddedSpringConstraintFacade& SpringConstraintFacade);

		UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(VertexFaceRepulsionStiffness, float);
		UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(VertexFaceMaxRepulsionIters, int32);
	};

}  // End namespace Chaos::Softs

