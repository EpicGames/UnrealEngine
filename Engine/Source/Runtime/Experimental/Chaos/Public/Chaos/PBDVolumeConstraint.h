// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// HEADER_UNIT_SKIP - Internal

#include "Chaos/PBDVolumeConstraintBase.h"

namespace Chaos::Softs
{

class FPBDVolumeConstraint : public FPBDVolumeConstraintBase
{
	typedef FPBDVolumeConstraintBase Base;

  public:
	  FPBDVolumeConstraint(const FSolverParticles& InParticles, TArray<TVec3<int32>>&& InConstraints, const FSolverReal InStiffness = (FSolverReal)1.)
	    : Base(InParticles, MoveTemp(InConstraints), InStiffness) {}
	virtual ~FPBDVolumeConstraint() override = default;

	void Apply(FSolverParticles& InParticles, const FSolverReal dt) const
	{
		const TArray<FSolverReal> W = Base::GetWeights(InParticles, (FSolverReal)1.);
		const TArray<FSolverVec3> Grads = Base::GetGradients(InParticles);
		const FSolverReal S = Base::GetScalingFactor(InParticles, Grads, W);
		for (uint32 i = 0; i < InParticles.Size(); ++i)
		{
			InParticles.P(i) -= S * W[i] * Grads[i];
		}
	}
};

}  // End namespace Chaos::Softs
