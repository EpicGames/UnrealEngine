// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// HEADER_UNIT_SKIP - Internal

#include "Chaos/PBDSoftsEvolutionFwd.h"
#include "Chaos/Array.h"
#include "Chaos/PBDParticles.h"
#include "Chaos/ParticleRule.h"

namespace Chaos::Softs
{

class FPBDVolumeConstraintBase
{
  public:
	  FPBDVolumeConstraintBase(const FSolverParticles& InParticles, TArray<TVec3<int32>>&& InConstraints, const FSolverReal InStiffness = (FSolverReal)1.)
	    : Constraints(InConstraints), Stiffness(InStiffness)
	{
		FSolverVec3 Com = FSolverVec3(0.);
		for (uint32 i = 0; i < InParticles.Size(); ++i)
		{
			Com += InParticles.GetX(i);
		}
		Com /= (FSolverReal)InParticles.Size();
		RefVolume = (FSolverReal)0.;
		for (const TVec3<int32>& Constraint : Constraints)
		{
			const FSolverVec3& P1 = InParticles.GetX(Constraint[0]);
			const FSolverVec3& P2 = InParticles.GetX(Constraint[1]);
			const FSolverVec3& P3 = InParticles.GetX(Constraint[2]);
			RefVolume += GetVolume(P1, P2, P3, Com);
		}
		RefVolume /= (FSolverReal)9.;
	}
	virtual ~FPBDVolumeConstraintBase() = default;

	TArray<FSolverReal> GetWeights(const FSolverParticles& InParticles, const FSolverReal Alpha) const
	{
		TArray<FSolverReal> W;
		W.SetNum(InParticles.Size());
		const FSolverReal OneMinusAlpha = (FSolverReal)1. - Alpha;
		const FSolverReal Wg = (FSolverReal)1. / (FSolverReal)InParticles.Size();
		FSolverReal WlDenom = (FSolverReal)0.;
		for (uint32 i = 0; i < InParticles.Size(); ++i)
		{
			WlDenom += (InParticles.P(i) - InParticles.GetX(i)).Size();
		}
		for (uint32 i = 0; i < InParticles.Size(); ++i)
		{
			const FSolverReal Wl = (InParticles.P(i) - InParticles.GetX(i)).Size() / WlDenom;
			W[i] = OneMinusAlpha * Wl + Alpha * Wg;
		}
		return W;
	}

	TArray<FSolverVec3> GetGradients(const FSolverParticles& InParticles) const
	{
		FSolverVec3 Com = FSolverVec3(0.);
		for (uint32 i = 0; i < InParticles.Size(); ++i)
		{
			Com += InParticles.P(i);
		}
		Com /= (FSolverReal)InParticles.Size();
		TArray<FSolverVec3> Grads;
		Grads.SetNum(InParticles.Size());
		for (FSolverVec3& Elem : Grads)
		{
			Elem = FSolverVec3(0.);
		}
		for (int32 i = 0; i < Constraints.Num(); ++i)
		{
			const TVec3<int32>& Constraint = Constraints[i];
			const int32 i1 = Constraint[0];
			const int32 i2 = Constraint[1];
			const int32 i3 = Constraint[2];
			const FSolverVec3& P1 = InParticles.P(i1);
			const FSolverVec3& P2 = InParticles.P(i2);
			const FSolverVec3& P3 = InParticles.P(i3);
			const FSolverReal Area = GetArea(P1, P2, P3);
			const FSolverVec3 Normal = GetNormal(P1, P2, P3, Com);
			Grads[i1] += Area * Normal;
			Grads[i2] += Area * Normal;
			Grads[i3] += Area * Normal;
		}
		for (FSolverVec3& Elem : Grads)
		{
			Elem *= (FSolverReal)1. / (FSolverReal)3.;
		}
		return Grads;
	}

	FSolverReal GetScalingFactor(const FSolverParticles& InParticles, const TArray<FSolverVec3>& Grads, const TArray<FSolverReal>& W) const
	{
		FSolverVec3 Com = FSolverVec3(0.);
		for (uint32 i = 0; i < InParticles.Size(); ++i)
		{
			Com += InParticles.P(i);
		}
		Com /= (FSolverReal)InParticles.Size();
		FSolverReal Volume = (FSolverReal)0.;
		for (const TVec3<int32>& Constraint : Constraints)
		{
			const FSolverVec3& P1 = InParticles.P(Constraint[0]);
			const FSolverVec3& P2 = InParticles.P(Constraint[1]);
			const FSolverVec3& P3 = InParticles.P(Constraint[2]);
			Volume += GetVolume(P1, P2, P3, Com);
		}
		Volume /= (FSolverReal)9.;
		FSolverReal Denom = (FSolverReal)0;
		for (uint32 i = 0; i < InParticles.Size(); ++i)
		{
			Denom += W[i] * Grads[i].SizeSquared();
		}
		const FSolverReal S = (Volume - RefVolume) / Denom;
		return Stiffness * S;
	}

	void SetStiffness(FSolverReal InStiffness) { Stiffness = FMath::Clamp(InStiffness, (FSolverReal)0., (FSolverReal)1.); }

protected:
	TArray<TVec3<int32>> Constraints;

private:
	// Utility functions for the triangle concept
	FSolverVec3 GetNormal(const FSolverVec3 P1, const FSolverVec3& P2, const FSolverVec3& P3, const FSolverVec3& Com) const
	{
		const FSolverVec3 Normal = FSolverVec3::CrossProduct(P2 - P1, P3 - P1).GetSafeNormal();
		if (FSolverVec3::DotProduct((P1 + P2 + P3) / (FSolverReal)3. - Com, Normal) < (FSolverReal)0.)
		{
			return -Normal;
		}
		return Normal;
	}

	FSolverReal GetArea(const FSolverVec3& P1, const FSolverVec3& P2, const FSolverVec3& P3) const
	{
		const FSolverVec3 B = (P2 - P1).GetSafeNormal();
		const FSolverVec3 H = FSolverVec3::DotProduct(B, P3 - P1) * B + P1;
		return (FSolverReal)0.5 * (P2 - P1).Size() * (P3 - H).Size();
	}

	FSolverReal GetVolume(const FSolverVec3& P1, const FSolverVec3& P2, const FSolverVec3& P3, const FSolverVec3& Com) const
	{
		return GetArea(P1, P2, P3) * FSolverVec3::DotProduct(P1 + P2 + P3, GetNormal(P1, P2, P3, Com));
	}

	FSolverReal RefVolume;
	FSolverReal Stiffness;
};

}  // End namespace Chaos::Softs
