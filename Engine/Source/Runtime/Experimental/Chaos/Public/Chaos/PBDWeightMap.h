// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/PBDSoftsEvolutionFwd.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainersFwd.h"
#include "ChaosStats.h"

DECLARE_CYCLE_STAT(TEXT("Chaos PBD WeightMap Apply Values"), STAT_PBD_WeightMapApplyValues, STATGROUP_Chaos);

namespace Chaos::Softs
{

/**
 * Weight map class for managing real time update to the weight map and low/high value ranges
 */
class FPBDWeightMap
{
public:

	/**
	 * Weightmap particle constructor. 
	 */
	inline FPBDWeightMap(
		const FSolverVec2& InWeightedValue,
		const TConstArrayView<FRealSingle>& Multipliers = TConstArrayView<FRealSingle>(),
		int32 ParticleCount = 0,  // A value of 0 also disables the map
		int32 TableSize = 16);  // Size of the lookup table, can't be more than 256 values, the larger the table the longer it takes to apply changes to the stiffness values;

	/**
	 * Weightmap constraint constructor. 
	 */
	template<int32 Valence>
	inline FPBDWeightMap(
		const FSolverVec2& InWeightedValue,
		const TConstArrayView<FRealSingle>& Multipliers = TConstArrayView<FRealSingle>(),
		const TConstArrayView<TVector<int32, Valence>>& Constraints = TConstArrayView<TVector<int32, Valence>>(),
		int32 ParticleOffset = INDEX_NONE, // Constraints have usually a particle offset added to them compared to the weight maps that always starts at index 0
		int32 ParticleCount = 0,  // A value of 0 also disables the map
		int32 TableSize = 16,  // Size of the lookup table, can't be more than 256 values, the larger the table the longer it takes to apply changes to the stiffness values
		typename TEnableIf<Valence >= 2 && Valence <= 4>::Type* = nullptr);  // Prevents incorrect valence, the value is actually unused

	virtual ~FPBDWeightMap() = default;

	FPBDWeightMap(const FPBDWeightMap&) = default;
	FPBDWeightMap(FPBDWeightMap&&) = default;
	FPBDWeightMap& operator=(const FPBDWeightMap&) = default;
	FPBDWeightMap& operator=(FPBDWeightMap&&) = default;

	/** Return the number of values stored in the weight map. */
	int32 Num() const { return Indices.Num(); }

	/** Return whether this object contains weight map values. */
	bool HasWeightMap() const { return Table.Num() > 1 && FMath::Abs(WeightedValue[0] - WeightedValue[1]) > UE_KINDA_SMALL_NUMBER; }

	/**
	 * Set the low and high values of the weight map.
	 * The weight map table only gets updated after ApplyValues is called.
	 * Low and high values are clamped between [0,1]
	 */
	void SetWeightedValue(const FSolverVec2& InWeightedValue) { bIsDirty |= WeightedValue != InWeightedValue; WeightedValue = InWeightedValue;}

	/**
	 * Return the low and high values set for this weight map.
	 * Both values will always be between 0 and 1 due to having been clamped in SetWeightedValue.
	 */
	const FSolverVec2& GetWeightedValue() const { return WeightedValue; }

	/** Update the weight map table with the current simulation parameters. */
	void ApplyValues(bool* bOutUpdated = nullptr) { ApplyValues([](FSolverReal Value)->FSolverReal { return Value; }, bOutUpdated); }

	/**
	 * Lookup for the exponential weighted value at the specified weight map index.
	 * This function will assert if it is called with a non zero index on an empty weight map.
	*/
	FSolverReal operator[](int32 Index) const { return Table[Indices[Index]]; }

	/** Return the exponential value at the Low weight. */
	FSolverReal GetLow() const { return Table[0]; }

	/** Return the exponential value at the High weight. */
	FSolverReal GetHigh() const { return Table.Last(); }

	/** Return the exponential stiffness value when the weight map is not used. */
	explicit operator FSolverReal() const { return GetLow(); }

	FSolverReal GetValue(int32 Index) const
	{
		if (HasWeightMap())
		{
			return (*this)[Index];
		}
		return GetLow();
	}

	/** Return the particles/constraints indices to the stiffness table as a read only array. */
	TConstArrayView<uint8> GetIndices() const { return TConstArrayView<uint8>(Indices); }

	/** Return the table of stiffnesses as a read only array. */
	TConstArrayView<FSolverReal> GetTable() const { return TConstArrayView<FSolverReal>(Table); }

	/** Reorder Indices based on Constraint reordering. */
	inline void ReorderIndices(const TArray<int32>& OrigToReorderedIndices);

protected:
	template<typename FunctorType>
	inline void ApplyValues(FunctorType&& MappingFunction, bool* bOutUpdated = nullptr);

	TArray<uint8> Indices; // Per particle/constraints array of index to the stiffness table
	TArray<FSolverReal> Table;  // Fixed lookup table of stiffness values, use uint8 indexation
	FSolverVec2 WeightedValue;
	bool bIsDirty = true;  // Whether the values have changed and the table needs updating
};

inline FPBDWeightMap::FPBDWeightMap(
	const FSolverVec2& InWeightedValue,
	const TConstArrayView<FRealSingle>& Multipliers,
	int32 ParticleCount,
	int32 TableSize)
	: WeightedValue(InWeightedValue)
{
	check(TableSize > 0 && TableSize < 256);  // The Stiffness lookup table is restricted to uint8 sized indices

	if (Multipliers.Num() == ParticleCount && ParticleCount > 0)
	{
		// Convert the weight maps into an array of lookup indices to the stiffness table
		Indices.AddUninitialized(ParticleCount);

		const FRealSingle TableScale = (FRealSingle)(TableSize - 1);

		for (int32 Index = 0; Index < ParticleCount; ++Index)
		{
			Indices[Index] = (uint8)(FMath::Clamp(Multipliers[Index], (FRealSingle)0., (FRealSingle)1.) * TableScale);
		}

		// Initialize empty table until ApplyValues is called
		Table.AddZeroed(TableSize);
	}
	else
	{
		// Initialize with a one element table until ApplyValues is called
		Indices.AddZeroed(1);
		Table.AddZeroed(1);
	}
}

template<int32 Valence>
inline FPBDWeightMap::FPBDWeightMap(
	const FSolverVec2& InWeightedValue,
	const TConstArrayView<FRealSingle>& Multipliers,
	const TConstArrayView<TVector<int32, Valence>>& Constraints,
	int32 ParticleOffset,
	int32 ParticleCount,
	int32 TableSize,
	typename TEnableIf<Valence >= 2 && Valence <= 4>::Type*)
	: WeightedValue(InWeightedValue)
{
	check(TableSize > 0 && TableSize < 256);  // The Stiffness lookup table is restricted to uint8 sized indices

	const int32 ConstraintCount = Constraints.Num();

	if (Multipliers.Num() == ParticleCount && ParticleCount > 0 && ConstraintCount > 0)
	{
		// Convert the weight maps into an array of lookup indices to the stiffness table
		Indices.AddUninitialized(ConstraintCount);

		const FRealSingle TableScale = (FRealSingle)(TableSize - 1);

		for (int32 ConstraintIndex = 0; ConstraintIndex < ConstraintCount; ++ConstraintIndex)
		{
			const TVector<int32, Valence>& Constraint = Constraints[ConstraintIndex];

			FRealSingle Weight = 0.f;
			for (int32 Index = 0; Index < Valence; ++Index)
			{
				Weight += FMath::Clamp(Multipliers[Constraint[Index] - ParticleOffset], (FRealSingle)0., (FRealSingle)1.);
			}
			Weight /= (FRealSingle)Valence;

			Indices[ConstraintIndex] = (uint8)(Weight * TableScale);
		}

		// Initialize empty table until ApplyValues is called
		Table.AddZeroed(TableSize);
	}
	else
	{
		// Initialize with a one element table until ApplyValues is called
		Indices.AddZeroed(1);
		Table.AddZeroed(1);
	}
}

template<typename FunctorType>
inline void FPBDWeightMap::ApplyValues(FunctorType&& MappingFunction, bool* bOutUpdated)
{
	SCOPE_CYCLE_COUNTER(STAT_PBD_WeightMapApplyValues);
	if (bOutUpdated)
	{
		*bOutUpdated = true;
	}
	if (bIsDirty)
	{
		const FSolverReal Offset = WeightedValue[0];
		const FSolverReal Range = WeightedValue[1] - WeightedValue[0];
		const int32 TableSize = Table.Num();
		const FSolverReal WeightIncrement = (TableSize > 1) ? (FSolverReal)1. / (FSolverReal)(TableSize - 1) : (FSolverReal)1.; // Must allow full range from 0 to 1 included
		for (int32 Index = 0; Index < TableSize; ++Index)
		{
			const FSolverReal Weight = (FSolverReal)Index * WeightIncrement;
			Table[Index] = MappingFunction(Offset + Weight * Range);
		}

		bIsDirty = false;
	}
}

inline void FPBDWeightMap::ReorderIndices(const TArray<int32>& OrigToReorderedConstraintIndices)
{
	if (Indices.Num() == OrigToReorderedConstraintIndices.Num())
	{
		TArray<uint8> ReorderedIndices;
		ReorderedIndices.SetNumUninitialized(Indices.Num());
		for (int32 OrigConstraintIndex = 0; OrigConstraintIndex < OrigToReorderedConstraintIndices.Num(); ++OrigConstraintIndex)
		{
			const int32 ReorderedConstraintIndex = OrigToReorderedConstraintIndices[OrigConstraintIndex];
			ReorderedIndices[ReorderedConstraintIndex] = Indices[OrigConstraintIndex];
		}
		Indices = MoveTemp(ReorderedIndices);
	}
	else
	{
		check(Indices.Num() == 1);
	}
}

}  // End namespace Chaos::Softs
