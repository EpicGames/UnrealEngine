// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Defines.h"

#if !COMPILE_WITHOUT_UNREAL_SUPPORT
#include "Async/ParallelFor.h"
#else
#include <functional>

// TODO(mlentine): Add parallel support
static void ParallelFor(const int32 Size, std::function<void(int32)> PerParticleFunction)
{
	for (int32 i = 0; i < Size; ++i)
	{
		PerParticleFunction(i);
	}
}
#endif
