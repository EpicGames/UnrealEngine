// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"


namespace Chaos
{
	enum class EObjectStateType : int8
	{
		Uninitialized = 0,
		Sleeping = 1,
		Kinematic = 2,
		Static = 3,
		Dynamic = 4,

		Count
	};
}