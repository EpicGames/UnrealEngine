// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/ParticleHandle.h"
#include "Chaos/Utilities.h"

//
// These accessors existed to allow us to switch between particles whose transforms are maintained
// in center of mass space, versus actor space. The should no longer be needed - we have settled on Actor Space. 
// See particle handle methods like GetTransformPQ(), SetTransformPQCom() etc.
//
// @todo(chaos): deprecate this file and its contents
//

#define CHAOS_PARTICLE_ACTORTRANSFORM 1

namespace Chaos
{
	class FParticleSpatialAccessorXR
	{
	public:
		template<typename T_PARTICLEHANDLE>
		static inline FVec3& Position(T_PARTICLEHANDLE Particle) { return Particle->GetX(); }
		template<typename T_PARTICLEHANDLE>
		static inline const FVec3& GetPosition(T_PARTICLEHANDLE Particle) { return Particle->GetX(); }
		template<typename T_PARTICLEHANDLE>
		UE_DEPRECATED(5.4, "Use Particle->SetR() instead.")
		static inline FRotation3 Rotation(T_PARTICLEHANDLE Particle) { return Particle->GetR(); }
		template<typename T_PARTICLEHANDLE>
		static inline const FRotation3 GetRotation(T_PARTICLEHANDLE Particle) { return Particle->GetR(); }

		static inline const FVec3& Position(TPBDRigidParticles<FReal, 3>& Particles, const int32 Index) { return Particles.GetX(Index); }
		static inline const FRotation3 Rotation(TPBDRigidParticles<FReal, 3>& Particles, const int32 Index) { return Particles.GetR(Index); }
	};

	class FParticleSpatialAccessorPQ
	{
	public:
		template<typename T_PARTICLEHANDLE>
		static inline FVec3& Position(T_PARTICLEHANDLE Particle) { return Particle->P(); }
		template<typename T_PARTICLEHANDLE>
		static inline const FVec3& GetPosition(T_PARTICLEHANDLE Particle) { return Particle->GetP(); }
		template<typename T_PARTICLEHANDLE>
		static inline FRotation3& Rotation(T_PARTICLEHANDLE Particle) { return Particle->GetQ(); }
		template<typename T_PARTICLEHANDLE>
		static inline const FRotation3 GetRotation(T_PARTICLEHANDLE Particle) { return Particle->GetQ(); }

		static inline const FVec3& Position(TPBDRigidParticles<FReal, 3>& Particles, const int32 Index) { return Particles.GetP(Index); }
		static inline FRotation3 Rotation(TPBDRigidParticles<FReal, 3>& Particles, const int32 Index) { return Particles.GetQ(Index); }
	};

	/**
	 * Particle Space == Actor Space (Transforms)
	 * Velocities in CoM Space.
	 */
	template <typename TSpatialAccessor>
	class FParticleUtilities_ActorSpace
	{
	public:
		template<typename T_PARTICLEHANDLE>
		static inline FRigidTransform3 GetActorWorldTransform(T_PARTICLEHANDLE Particle)
		{
			return FRigidTransform3(TSpatialAccessor::GetPosition(Particle), TSpatialAccessor::GetRotation(Particle));
		}

		template<typename T_PARTICLEHANDLE>
		static inline void SetActorWorldTransform(T_PARTICLEHANDLE Particle, const FRigidTransform3& ActorWorldT)
		{
			TSpatialAccessor::Position(Particle) = ActorWorldT.GetTranslation();
			TSpatialAccessor::Rotation(Particle) = ActorWorldT.GetRotation();
		}

		template<typename T_PARTICLEHANDLE>
		static inline const FRigidTransform3& ActorLocalToParticleLocal(T_PARTICLEHANDLE Particle, const FRigidTransform3& ActorLocalT)
		{
			return ActorLocalT;
		}

		template<typename T_PARTICLEHANDLE>
		static inline const FRigidTransform3& ActorWorldToParticleWorld(T_PARTICLEHANDLE Particle, const FRigidTransform3& ActorWorldT)
		{
			return ActorWorldT;
		}

		template<typename T_PARTICLEHANDLE>
		static inline FMatrix33 GetWorldInertia(T_PARTICLEHANDLE Particle)
		{
			return Utilities::ComputeWorldSpaceInertia(TSpatialAccessor::GetRotation(Particle) * Particle->RotationOfMass(), Particle->I());
		}

		template<typename T_PARTICLEHANDLE>
		static inline FMatrix33 GetWorldInvInertia(T_PARTICLEHANDLE Particle)
		{
			return Utilities::ComputeWorldSpaceInertia(TSpatialAccessor::GetRotation(Particle) * Particle->RotationOfMass(), Particle->InvI());
		}

		/**
		 * Convert an particle position into a com-local-space position
		 */
		template<typename T_PARTICLEHANDLE>
		static inline FVec3 ParticleLocalToCoMLocal(T_PARTICLEHANDLE Particle, const FVec3& P)
		{
			return Particle->RotationOfMass().UnrotateVector(P - Particle->CenterOfMass());
		}

		/**
		 * Convert a particle rotation into a com-local-space rotation
		 */
		template<typename T_PARTICLEHANDLE>
		static inline FRotation3 ParticleLocalToCoMLocal(T_PARTICLEHANDLE Particle, const FRotation3& Q)
		{
			return Particle->RotationOfMass().Inverse()* Q;
		}

		/**
		 * Convert an particle-local-space transform into a com-local-space transform
		 */
		template<typename T_PARTICLEHANDLE>
		static inline FRigidTransform3 ParticleLocalToCoMLocal(T_PARTICLEHANDLE Particle, const FRigidTransform3& T)
		{
			return FRigidTransform3(ParticleLocalToCoMLocal(Particle, T.GetTranslation()), ParticleLocalToCoMLocal(Particle, T.GetRotation()));
		}

		/**
		 * Get the velocity at point 'RelPos', where 'RelPos' is a world-space position relative to the Particle's center of mass.
		 */
		template<typename T_PARTICLEHANDLE>
		static inline FVec3 GetVelocityAtCoMRelativePosition(T_PARTICLEHANDLE Particle, const FVec3& RelPos)
		{
			return Particle->V() + FVec3::CrossProduct(Particle->W(), RelPos);
		}

		template<typename T_PARTICLEHANDLE>
		static inline FVec3 GetPreviousVelocityAtCoMRelativePosition(T_PARTICLEHANDLE Particle, const FVec3& RelPos)
		{
			return Particle->PreV() + FVec3::CrossProduct(Particle->PreW(), RelPos);
		}

		template<typename T_PARTICLEHANDLE>
		static inline FVec3 GetCoMWorldPosition(T_PARTICLEHANDLE Particle)
		{
			return TSpatialAccessor::GetPosition(Particle) + TSpatialAccessor::GetRotation(Particle).RotateVector(Particle->CenterOfMass());
		}

		static inline FVec3 GetCoMWorldPosition(TPBDRigidParticles<FReal, 3>& Particles, const int32 Index)
		{
			return TSpatialAccessor::Position(Particles, Index) + TSpatialAccessor::Rotation(Particles, Index).RotateVector(Particles.CenterOfMass(Index));
		}

		template<typename T_PARTICLEHANDLE>
		static inline FRotation3 GetCoMWorldRotation(T_PARTICLEHANDLE Particle)
		{
			return TSpatialAccessor::GetRotation(Particle) * Particle->RotationOfMass();
		}
	
		static inline FRotation3 GetCoMWorldRotation(TPBDRigidParticles<FReal, 3>& Particles, const int32 Index)
		{
			return TSpatialAccessor::Rotation(Particles, Index) * Particles.RotationOfMass(Index);
		}

		template<typename T_PARTICLEHANDLE>
		static inline FRigidTransform3 GetCoMWorldTransform(T_PARTICLEHANDLE Particle)
		{
			return FRigidTransform3(GetCoMWorldPosition(Particle), GetCoMWorldRotation(Particle));
		}

		/**
		 * Update the particle's position and rotation by specifying a new center of mass transform.
		 */
		template<typename T_PARTICLEHANDLE>
		static inline void SetCoMWorldTransform(T_PARTICLEHANDLE Particle, const FVec3& PCoM, const FRotation3& QCoM)
		{
			const FRotation3 Q = QCoM * Particle->RotationOfMass().Inverse();
			const FVec3 P = PCoM - Q.RotateVector(Particle->CenterOfMass());
			
			TSpatialAccessor::Position(Particle) = P;
			TSpatialAccessor::Rotation(Particle) = Q;
		}

		static inline void SetCoMWorldTransform(TPBDRigidParticles<FReal, 3>& Particles, const int32 Index, const FVec3& PCoM, const FRotation3& QCoM)
		{
			const FRotation3 Q = QCoM * Particles.RotationOfMass(Index).Inverse();
			const FVec3 P = PCoM - Q.RotateVector(Particles.CenterOfMass(Index));

			TSpatialAccessor::Position(Particles, Index) = P;
			TSpatialAccessor::Rotation(Particles, Index) = Q;
		}
		
		template<typename T_PARTICLEHANDLE>
		static inline void AddForceAtPositionLocal(T_PARTICLEHANDLE Particle, const FVec3& LocalForce, const FVec3& LocalPosition)
		{
			const FRigidTransform3 ParticleTransform = GetActorWorldTransform(Particle);
			const FVec3 WorldPosition = ParticleTransform.TransformPosition(LocalPosition);
			const FVec3 WorldForce = ParticleTransform.TransformVector(LocalForce);

			AddForceAtPositionWorld(Particle, WorldForce, WorldPosition);
		}

		template<typename T_PARTICLEHANDLE>
		static inline void AddForceAtPositionWorld(T_PARTICLEHANDLE Particle, const FVec3& Force, const FVec3& Position)
		{
			const FVec3 WorldCOM = GetCoMWorldPosition(Particle);
			const FVec3 Torque = FVec3::CrossProduct(Position - WorldCOM, Force);

			Particle->AddForce(Force);
			Particle->AddTorque(Torque);
		}
	};

	using FParticleUtilitiesPQ = FParticleUtilities_ActorSpace<FParticleSpatialAccessorPQ>;
	using FParticleUtilitiesXR = FParticleUtilities_ActorSpace<FParticleSpatialAccessorXR>;
	using FParticleUtilities = FParticleUtilities_ActorSpace<FParticleSpatialAccessorPQ>;
	using FParticleUtilitiesGT = FParticleUtilities_ActorSpace<FParticleSpatialAccessorXR>;

}