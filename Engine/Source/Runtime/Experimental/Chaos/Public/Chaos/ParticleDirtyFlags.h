// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/Core.h"
#include "Chaos/ChaosArchive.h"
#include "Chaos/Character/CharacterGroundConstraintSettings.h"
#include "Chaos/Box.h"
#include "Chaos/Particles.h"
#include "Chaos/PhysicalMaterials.h"
#include "Chaos/GeometryParticlesfwd.h"
#include "Chaos/CollisionFilterData.h"
#include "Chaos/Collision/CollisionConstraintFlags.h"
#include "Chaos/KinematicTargets.h"
#include "Chaos/RigidParticleControlFlags.h"
#include "UObject/ExternalPhysicsCustomObjectVersion.h"
#include "UObject/ExternalPhysicsMaterialCustomObjectVersion.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/FortniteSeasonBranchObjectVersion.h"
#include "UObject/PhysicsObjectVersion.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"
#include "Framework/PhysicsProxyBase.h"
#include "Chaos/Evolution/IterationSettings.h"
#include "PBDJointConstraintTypes.h"
#include "PBDSuspensionConstraintTypes.h"

#ifndef CHAOS_DEBUG_NAME
#define CHAOS_DEBUG_NAME 0
#endif

class FName;

namespace Chaos
{

struct FParticleID
{
	int32 GlobalID = INDEX_NONE; //Set by global ID system
	int32 LocalID = INDEX_NONE;	//Set by local client. This can only be used in cases where the LocalID will be set in the same way (for example we always spawn N client only particles)

	bool operator<(const FParticleID& Other) const
	{
		if (GlobalID == Other.GlobalID)
		{
			return LocalID < Other.LocalID;
		}
		return GlobalID < Other.GlobalID;
	}

	bool operator==(const FParticleID& Other) const
	{
		return GlobalID == Other.GlobalID && LocalID == Other.LocalID;
	}
};

enum class EResimType: uint8;
enum class ESleepType: uint8;

class FParticlePositionRotation
{
public:
	void Serialize(FChaosArchive& Ar)
	{
		Ar.UsingCustomVersion(FFortniteReleaseBranchCustomObjectVersion::GUID);

		if (Ar.CustomVer(FFortniteReleaseBranchCustomObjectVersion::GUID) >= FFortniteReleaseBranchCustomObjectVersion::SinglePrecisonParticleData)
		{
			Ar << MX << MR;
		}
		else
		{
			FRotation3 DoublePrecisionRotation = FRotation3(MR);
			Ar << MX << DoublePrecisionRotation;
			MR = FRotation3f(MR);
		}
		
	}

	template <typename TOther>
	void CopyFrom(const TOther& Other)
	{
		MX = Other.GetX();
		MR = FRotation3f(Other.GetR());
	}

	template <typename TOther>
	bool IsEqual(const TOther& Other) const
	{
		return MX == Other.X() && MR == FRotation3f(Other.R());
	}

	template <typename TOther>
	bool IsNearlyEqual(const TOther& Other, FReal Leniency = UE_KINDA_SMALL_NUMBER) const
	{
		return FVec3::IsNearlyEqual(MX, Other.X(), Leniency) && FRotation3f::IsNearlyEqual(MR, FRotation3f(Other.R()), static_cast<FRealSingle>(Leniency));
	}

	bool operator==(const FParticlePositionRotation& Other) const
	{
		return IsEqual(Other);
	}

	const FVec3& X() const { return MX; }
	const FVec3& GetX() const { return MX; }
	void SetX(const FVec3& InX){ MX = InX; }

	const FRotation3 R() const { return FRotation3(MR); }
	const FRotation3 GetR() const { return FRotation3(MR); }
	void SetR(const FRotation3& InR) { MR = FRotation3f(InR); }
	
private:
	FVec3 MX;
	FRotation3f MR;

};

inline FChaosArchive& operator<<(FChaosArchive& Ar,FParticlePositionRotation& Data)
{
	Data.Serialize(Ar);
	return Ar;
}

class FParticleVelocities
{
public:
	void Serialize(FChaosArchive& Ar)
	{
		Ar.UsingCustomVersion(FFortniteReleaseBranchCustomObjectVersion::GUID);

		if (Ar.CustomVer(FFortniteReleaseBranchCustomObjectVersion::GUID) >= FFortniteReleaseBranchCustomObjectVersion::SinglePrecisonParticleData)
		{
			Ar << MV << MW;
		}
		else
		{
			FVec3 MVDouble(MV);
			FVec3 MWDouble(MW);
			Ar << MVDouble << MWDouble;
			MV = FVec3f(MVDouble);
			MW = FVec3f(MWDouble);
		}

	}

	template <typename TOther>
	void CopyFrom(const TOther& Other)
	{
		MV = Other.GetV();
		MW = Other.GetW();
	}

	template <typename TOther>
	bool IsEqual(const TOther& Other) const
	{
		return MV == FVec3f(Other.GetV()) && MW == FVec3f(Other.GetW());
	}

	template <typename TOther>
	bool IsNearlyEqual(const TOther& Other, FReal Leniency = UE_KINDA_SMALL_NUMBER) const
	{
		return FVec3::IsNearlyEqual(MV, Other.GetV(), Leniency) && FVec3::IsNearlyEqual(MW, Other.GetW(), Leniency);
	}

	bool operator==(const FParticleVelocities& Other) const
	{
		return IsEqual(Other);
	}

	const FVec3 V() const { return FVec3(MV); }
	const FVec3 GetV() const { return FVec3(MV); }
	void SetV(const FVec3& V) { MV = FVec3f(V); }

	const FVec3 W() const { return FVec3(MW); }
	const FVec3 GetW() const { return FVec3(MW); }
	void SetW(const FVec3& W){ MW = FVec3f(W); }

private:
	FVec3f MV;
	FVec3f MW;
};

inline FChaosArchive& operator<<(FChaosArchive& Ar,FParticleVelocities& Data)
{
	Data.Serialize(Ar);
	return Ar;
}

class FParticleDynamics
{
public:
	void Serialize(FChaosArchive& Ar)
	{
		Ar.UsingCustomVersion(FFortniteReleaseBranchCustomObjectVersion::GUID);

		if (Ar.CustomVer(FFortniteReleaseBranchCustomObjectVersion::GUID) >= FFortniteReleaseBranchCustomObjectVersion::SinglePrecisonParticleData)
		{
			Ar << MAcceleration;
			Ar << MAngularAcceleration;
			Ar << MLinearImpulseVelocity;
			Ar << MAngularImpulseVelocity;
		}
		else
		{
			FVec3 AccelerationDouble(MAcceleration);
			FVec3 AngularAccelerationDouble(MAngularAcceleration);
			FVec3 LinearImpulseVelocityDouble(MLinearImpulseVelocity);
			FVec3 AngularImpulseVelocityDouble(MAngularImpulseVelocity);
			Ar << AccelerationDouble;
			Ar << AngularAccelerationDouble;
			Ar << LinearImpulseVelocityDouble;
			Ar << AngularImpulseVelocityDouble;
			MAcceleration = FVec3f(AccelerationDouble);
			MAngularAcceleration = FVec3f(AngularAccelerationDouble);
			MLinearImpulseVelocity = FVec3f(LinearImpulseVelocityDouble);
			MAngularImpulseVelocity = FVec3f(AngularImpulseVelocityDouble);
		}
	}

	template <typename TOther>
	void CopyFrom(const TOther& Other)
	{
		MAcceleration = Other.Acceleration();
		MAngularAcceleration = Other.AngularAcceleration();
		MLinearImpulseVelocity = Other.LinearImpulseVelocity();
		MAngularImpulseVelocity = Other.AngularImpulseVelocity();
	}

	template <typename TOther>
	bool IsEqual(const TOther& Other) const
	{
		return Acceleration() == Other.Acceleration()
			&& AngularAcceleration() == Other.AngularAcceleration()
			&& LinearImpulseVelocity() == Other.LinearImpulseVelocity()
			&& AngularImpulseVelocity() == Other.AngularImpulseVelocity();
	}

	template <typename TOther>
	bool IsNearlyEqual(const TOther& Other, FReal Leniency = UE_KINDA_SMALL_NUMBER) const
	{
		return FVec3::IsNearlyEqual(Acceleration(), Other.Acceleration(), Leniency)
			&& FVec3::IsNearlyEqual(AngularAcceleration(), Other.AngularAcceleration(), Leniency)
			&& FVec3::IsNearlyEqual(LinearImpulseVelocity(), Other.LinearImpulseVelocity(), Leniency)
			&& FVec3::IsNearlyEqual(AngularImpulseVelocity(), Other.AngularImpulseVelocity(), Leniency);
	}

	bool operator==(const FParticleDynamics& Other) const
	{
		return IsEqual(Other);
	}

	FVec3 Acceleration() const { return FVec3(MAcceleration); }
	void SetAcceleration(const FVec3& Acceleration){ MAcceleration = FVec3f(Acceleration); }

	FVec3 AngularAcceleration() const { return FVec3(MAngularAcceleration); }
	void SetAngularAcceleration(const FVec3& AngularAcceleration){ MAngularAcceleration = FVec3f(AngularAcceleration); }

	FVec3 LinearImpulseVelocity() const { return FVec3(MLinearImpulseVelocity); }
	void SetLinearImpulseVelocity(const FVec3& LinearImpulseVelocity){ MLinearImpulseVelocity = FVec3f(LinearImpulseVelocity); }

	FVec3 AngularImpulseVelocity() const { return FVec3(MAngularImpulseVelocity); }
	void SetAngularImpulseVelocity(const FVec3& AngularImpulseVelocity){ MAngularImpulseVelocity = FVec3f(AngularImpulseVelocity); }

	static FParticleDynamics ZeroValue()
	{
		FParticleDynamics Result;
		Result.MAcceleration = FVec3(0);
		Result.MAngularAcceleration = FVec3(0);
		Result.MLinearImpulseVelocity = FVec3(0);
		Result.MAngularImpulseVelocity = FVec3(0);

		return Result;
	}

private:
	FVec3f MAcceleration;
	FVec3f MAngularAcceleration;
	FVec3f MLinearImpulseVelocity;
	FVec3f MAngularImpulseVelocity;

};

typedef TVector<IPhysicsProxyBase*, 2> FProxyBasePair;

struct FProxyBasePairProperty
{
	FProxyBasePair ParticleProxies = { nullptr, nullptr };
};

struct FProxyBaseProperty
{
	IPhysicsProxyBase* Proxy = nullptr;
};

struct FPhysicsObject;
typedef TVector<FPhysicsObject*, 2> FPhysicsObjectPair;

struct FPhysicsObjectPairProperty
{
	FPhysicsObjectPair PhysicsBodies = { nullptr, nullptr };
};

struct FPhysicsObjectProperty
{
	FPhysicsObject* PhysicsBody = nullptr;
};

inline FChaosArchive& operator<<(FChaosArchive& Ar, FParticleDynamics& Data)
{
	Data.Serialize(Ar);
	return Ar;
}

class FParticleMassProps
{
public:
	void Serialize(FChaosArchive& Ar)
	{
		Ar << MCenterOfMass;
		Ar << MRotationOfMass;
		Ar << MI;
		Ar << MInvI;
		Ar << MM;
		Ar << MInvM;
	}

	template <typename TOther>
	void CopyFrom(const TOther& Other)
	{
		MCenterOfMass = Other.CenterOfMass();
		MRotationOfMass = Other.RotationOfMass();
		MI = Other.I();
		MInvI = Other.InvI();
		MM = Other.M();
		MInvM = Other.InvM();
	}

	template <typename TOther>
	bool IsEqual(const TOther& Other) const
	{
		return CenterOfMass() == Other.CenterOfMass()
			&& RotationOfMass() == Other.RotationOfMass()
			&& I() == Other.I()
			&& InvI() == Other.InvI()
			&& M() == Other.M()
			&& InvM() == Other.InvM();
	}

	template <typename TOther>
	bool IsNearlyEqual(const TOther& Other, FReal Leniency = UE_KINDA_SMALL_NUMBER) const
	{
		return FVec3::IsNearlyEqual(CenterOfMass(), Other.CenterOfMass(), Leniency)
			&& FRotation3::IsNearlyEqual(RotationOfMass(), Other.RotationOfMass(), static_cast<FRealSingle>(Leniency))
			&& FVec3::IsNearlyEqual(I(), Other.I(), Leniency)
			&& FVec3::IsNearlyEqual(InvI(), Other.InvI(), Leniency)
			&& FMath::Abs(M() - Other.M()) <= Leniency
			&& FMath::Abs(InvM() - Other.InvM()) <= Leniency;
	}

	bool operator==(const FParticleMassProps& Other) const
	{
		return IsEqual(Other);
	}

	const FVec3& CenterOfMass() const { return MCenterOfMass; }
	void SetCenterOfMass(const FVec3& InCenterOfMass){ MCenterOfMass = InCenterOfMass; }

	const FRotation3& RotationOfMass() const { return MRotationOfMass; }
	void SetRotationOfMass(const FRotation3& InRotationOfMass){ MRotationOfMass = InRotationOfMass; }

	const TVec3<FRealSingle>& I() const { return MI; }
	void SetI(const TVec3<FRealSingle>& InI){ MI = InI; }

	const TVec3<FRealSingle>& InvI() const { return MInvI; }
	void SetInvI(const TVec3<FRealSingle>& InInvI){ MInvI = InInvI; }

	FReal M() const { return MM; }
	void SetM(FReal InM){ MM = InM; }

	FReal InvM() const { return MInvM; }
	void SetInvM(FReal InInvM){ MInvM = InInvM; }

private:
	FVec3 MCenterOfMass;
	FReal MM;
	FRotation3 MRotationOfMass;
	TVec3<FRealSingle> MI;
	TVec3<FRealSingle> MInvI;
	FReal MInvM;


};

inline FChaosArchive& operator<<(FChaosArchive& Ar,FParticleMassProps& Data)
{
	Data.Serialize(Ar);
	return Ar;
}

class FParticleDynamicMisc
{
public:
	void Serialize(FChaosArchive& Ar)
	{
		Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);
		Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
		Ar.UsingCustomVersion(FExternalPhysicsCustomObjectVersion::GUID);
		Ar.UsingCustomVersion(FPhysicsObjectVersion::GUID);
		Ar.UsingCustomVersion(FFortniteReleaseBranchCustomObjectVersion::GUID);

		const bool bSinglePrecision = Ar.CustomVer(FFortniteReleaseBranchCustomObjectVersion::GUID) >= FFortniteReleaseBranchCustomObjectVersion::SinglePrecisonParticleData;

		if (bSinglePrecision)
		{
			Ar << MLinearEtherDrag;
			Ar << MAngularEtherDrag;
		}
		else
		{
			FReal LinearEtherDragDouble = FReal(MLinearEtherDrag);
			FReal AngularEtherDragDouble = FReal(MAngularEtherDrag);
			Ar << LinearEtherDragDouble;
			Ar << AngularEtherDragDouble;
			MLinearEtherDrag = FRealSingle(LinearEtherDragDouble);
			MAngularEtherDrag = FRealSingle(AngularEtherDragDouble);
		}
		
		Ar << MObjectState;

		// Flags moved into a bitmask
		const bool bAddControlFlags = (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) >= FUE5MainStreamObjectVersion::AddRigidParticleControlFlags);
		if (!bAddControlFlags && Ar.IsLoading())
		{
			bool bGravityEnabled;
			Ar << bGravityEnabled;
			MControlFlags.SetGravityEnabled(bGravityEnabled);
		}
		Ar << MSleepType;
		if (!bAddControlFlags && Ar.IsLoading())
		{
			bool bOneWayInteraction = false;
			if (Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) >= FExternalPhysicsCustomObjectVersion::AddOneWayInteraction)
			{
				Ar << bOneWayInteraction;
			}
			MControlFlags.SetOneWayInteractionEnabled(bOneWayInteraction);
		}

		if (!bAddControlFlags && Ar.IsLoading())
		{
			if (Ar.CustomVer(FPhysicsObjectVersion::GUID) >= FPhysicsObjectVersion::AddCCDEnableFlag)
			{
				bool bCCDEnabled;
				Ar << bCCDEnabled;
				MControlFlags.SetCCDEnabled(bCCDEnabled);
			}
		}

		const bool bAddCollisionConstraintFlagUE4 = (Ar.CustomVer(FPhysicsObjectVersion::GUID) >= FPhysicsObjectVersion::AddCollisionConstraintFlag);
		const bool bAddCollisionConstraintFlagUE5 = (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) >= FUE5MainStreamObjectVersion::AddCollisionConstraintFlag);		
		if (bAddCollisionConstraintFlagUE4 || bAddCollisionConstraintFlagUE5)
		{
			Ar << MCollisionConstraintFlag;
		}

		const bool bAddDisableFlagUE4 = (Ar.CustomVer(FPhysicsObjectVersion::GUID) >= FPhysicsObjectVersion::AddDisabledFlag);
		const bool bAddDisableFlagUE5 = (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) >= FUE5MainStreamObjectVersion::AddDisabledFlag);
		if (bAddDisableFlagUE4 || bAddDisableFlagUE5)
		{
			Ar << bDisabled;
		}
		
		const bool bAddChaosMaxLinearAngularSpeedUE4 = (Ar.CustomVer(FPhysicsObjectVersion::GUID) >= FPhysicsObjectVersion::AddChaosMaxLinearAngularSpeed);
		const bool bAddChaosMaxLinearAngularSpeedUE5 = (Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) >= FUE5ReleaseStreamObjectVersion::AddChaosMaxLinearAngularSpeed);
		if (bAddChaosMaxLinearAngularSpeedUE4 || bAddChaosMaxLinearAngularSpeedUE5)
		{
			if (bSinglePrecision)
			{
				Ar << MMaxLinearSpeedSq;
				Ar << MMaxAngularSpeedSq;
			}
			else
			{
				FReal MaxLinearSpeedSqDouble = FReal(MMaxLinearSpeedSq);
				FReal MaxAngularSpeedSqDouble = FReal(MMaxAngularSpeedSq);
				Ar << MaxLinearSpeedSqDouble;
				Ar << MaxAngularSpeedSqDouble;
				MMaxLinearSpeedSq = FRealSingle(MaxLinearSpeedSqDouble);
				MMaxAngularSpeedSq = FRealSingle(MaxAngularSpeedSqDouble);
			}
		}

		// @todo(chaos): add this
		//Ar << MInitialOverlapDepenetrationVelocity;
		//Ar << MSleepThresholdMultiplier;
		//Ar << MIterationSettings
		if (Ar.IsLoading())
		{
			MInitialOverlapDepenetrationVelocity = 0;
			MSleepThresholdMultiplier = 1.0f;
			MIterationSettings = MIterationSettings.MakeEmpty();
		}

		if (bAddControlFlags)
		{
			Ar << MControlFlags;
		}
	}

	template <typename TOther>
	void CopyFrom(const TOther& Other)
	{
		SetLinearEtherDrag(Other.LinearEtherDrag());
		SetAngularEtherDrag(Other.AngularEtherDrag());
		SetMaxLinearSpeedSq(Other.MaxLinearSpeedSq());
		SetMaxAngularSpeedSq(Other.MaxAngularSpeedSq());
		SetInitialOverlapDepenetrationVelocity(Other.InitialOverlapDepenetrationVelocity());
		SetSleepThresholdMultiplier(Other.SleepThresholdMultiplier());
		SetObjectState(Other.ObjectState());
		SetCollisionGroup(Other.CollisionGroup());
		SetSleepType(Other.SleepType());
		SetCollisionConstraintFlags(Other.CollisionConstraintFlags());
		SetControlFlags(Other.ControlFlags());
		SetIterationSettings(Other.IterationSettings());
		SetDisabled(Other.Disabled());
	}

	template <typename TOther>
	bool IsEqual(const TOther& Other) const
	{
		return ObjectState() == Other.ObjectState()
			&& MLinearEtherDrag == FRealSingle(Other.LinearEtherDrag())
			&& MAngularEtherDrag == FRealSingle(Other.AngularEtherDrag())
			&& MMaxLinearSpeedSq == FRealSingle(Other.MaxLinearSpeedSq())
			&& MMaxAngularSpeedSq == FRealSingle(Other.MaxAngularSpeedSq())
			&& InitialOverlapDepenetrationVelocity() == Other.InitialOverlapDepenetrationVelocity()
			&& SleepThresholdMultiplier() == Other.SleepThresholdMultiplier()
			&& CollisionGroup() == Other.CollisionGroup()
			&& SleepType() == Other.SleepType()
			&& CollisionConstraintFlags() == Other.CollisionConstraintFlags()
			&& ControlFlags() == Other.ControlFlags()
			&& IterationSettings() == Other.IterationSettings()
			&& Disabled() == Other.Disabled();
	}

	template <typename TOther>
	bool IsNearlyEqual(const TOther& Other, FReal Leniency = UE_KINDA_SMALL_NUMBER) const
	{
		return ObjectState() == Other.ObjectState()
			&& FMath::Abs(MLinearEtherDrag - Other.MLinearEtherDrag) <= Leniency
			&& FMath::Abs(MAngularEtherDrag - Other.MAngularEtherDrag) <= Leniency
			&& FMath::Abs(MMaxLinearSpeedSq - Other.MMaxLinearSpeedSq) <= Leniency
			&& FMath::Abs(MMaxAngularSpeedSq - Other.MMaxAngularSpeedSq) <= Leniency
			&& FMath::Abs(InitialOverlapDepenetrationVelocity() - Other.InitialOverlapDepenetrationVelocity()) <= Leniency
			&& FMath::Abs(SleepThresholdMultiplier() - Other.SleepThresholdMultiplier()) <= Leniency
			&& CollisionGroup() == Other.CollisionGroup()
			&& SleepType() == Other.SleepType()
			&& CollisionConstraintFlags() == Other.CollisionConstraintFlags()
			&& ControlFlags() == Other.ControlFlags()
			&& IterationSettings() == Other.IterationSettings()
			&& Disabled() == Other.Disabled();
	}

	bool operator==(const FParticleDynamicMisc& Other) const
	{
		return IsEqual(Other);
	}

	FReal LinearEtherDrag() const { return FReal(MLinearEtherDrag); }
	void SetLinearEtherDrag(FReal InLinearEtherDrag) { MLinearEtherDrag = FRealSingle(InLinearEtherDrag); }

	FReal AngularEtherDrag() const { return FReal(MAngularEtherDrag); }
	void SetAngularEtherDrag(FReal InAngularEtherDrag) { MAngularEtherDrag = FRealSingle(InAngularEtherDrag); }

	FReal MaxLinearSpeedSq() const { return FReal(MMaxLinearSpeedSq); }
	void SetMaxLinearSpeedSq(FReal InMaxLinearSpeed) { MMaxLinearSpeedSq = FRealSingle(InMaxLinearSpeed); }

	FReal MaxAngularSpeedSq() const { return FReal(MMaxAngularSpeedSq); }
	void SetMaxAngularSpeedSq(FReal InMaxAngularSpeed) { MMaxAngularSpeedSq = FRealSingle(InMaxAngularSpeed); }

	FRealSingle InitialOverlapDepenetrationVelocity() const { return MInitialOverlapDepenetrationVelocity; }
	void SetInitialOverlapDepenetrationVelocity(FRealSingle InVel) { MInitialOverlapDepenetrationVelocity = InVel; }

	FRealSingle SleepThresholdMultiplier() const { return MSleepThresholdMultiplier; }
	void SetSleepThresholdMultiplier(FRealSingle InSleepThresholdMultiplier) { MSleepThresholdMultiplier = InSleepThresholdMultiplier; }

	EObjectStateType ObjectState() const { return MObjectState; }
	void SetObjectState(EObjectStateType InState){ MObjectState = InState; }

	bool GravityEnabled() const { return MControlFlags.GetGravityEnabled(); }
	void SetGravityEnabled(bool bInGravity){ MControlFlags.SetGravityEnabled(bInGravity); }
	int32 GravityGroupIndex() const { return MControlFlags.GetGravityGroupIndex(); }
	void SetGravityGroupIndex(int32 InIndex) { MControlFlags.SetGravityGroupIndex(InIndex); }

	bool UpdateKinematicFromSimulation() const { return MControlFlags.GetUpdateKinematicFromSimulation(); }
	void SetUpdateKinematicFromSimulation(bool bUpdateKinematicFromSimulation) { MControlFlags.SetUpdateKinematicFromSimulation(bUpdateKinematicFromSimulation); }

	bool GyroscopicTorqueEnabled() const { return MControlFlags.GetGyroscopicTorqueEnabled(); }
	void SetGyroscopicTorqueEnabled(bool bInGyroscopicTorqueEnabled) { MControlFlags.SetGyroscopicTorqueEnabled(bInGyroscopicTorqueEnabled); }

	bool CCDEnabled() const { return MControlFlags.GetCCDEnabled(); }
	void SetCCDEnabled(bool bInCCDEnabled) { MControlFlags.SetCCDEnabled(bInCCDEnabled); }

	bool MACDEnabled() const { return MControlFlags.GetMACDEnabled(); }
	void SetMACDEnabled(bool bInMACDEnabled) { MControlFlags.SetMACDEnabled(bInMACDEnabled); }

	bool PartialIslandSleepAllowed() const { return MControlFlags.GetPartialIslandSleepAllowed(); }
	void SetPartialIslandSleepAllowed(bool bInPartialIslandSleepAllowed) { MControlFlags.SetPartialIslandSleepAllowed(bInPartialIslandSleepAllowed); }

	void SetPositionSolverIterationCount(const int32 PositionSolverIterationCountIn) { MIterationSettings.SetNumPositionIterations(PositionSolverIterationCountIn); }
	void SetVelocitySolverIterationCount(const int32 VelocitySolverIterationCountIn) { MIterationSettings.SetNumVelocityIterations(VelocitySolverIterationCountIn); }
	void SetProjectionSolverIterationCount(const int32 ProjectionSolverIterationCountIn) { MIterationSettings.SetNumProjectionIterations(ProjectionSolverIterationCountIn); }
	Private::FIterationSettings IterationSettings() const { return MIterationSettings; }
	void SetIterationSettings(const Private::FIterationSettings& SolverIterationSettings) { MIterationSettings = SolverIterationSettings; }

	bool Disabled() const { return bDisabled; }
	void SetDisabled(bool bInDisabled) { bDisabled = bInDisabled; }

	int32 CollisionGroup() const { return MCollisionGroup; }
	void SetCollisionGroup(int32 InGroup){ MCollisionGroup = InGroup; }

	ESleepType SleepType() const { return MSleepType; }
	void SetSleepType(ESleepType Type) { MSleepType = Type; }

	uint32 CollisionConstraintFlags() const { return MCollisionConstraintFlag; }
	void SetCollisionConstraintFlags(uint32 InCollisionConstraintFlag) { MCollisionConstraintFlag = InCollisionConstraintFlag; }
	void AddCollisionConstraintFlag(const ECollisionConstraintFlags Flag) { MCollisionConstraintFlag |= uint32(Flag); }
	void RemoveCollisionConstraintFlag(const ECollisionConstraintFlags Flag) { MCollisionConstraintFlag &= ~uint32(Flag); }

	bool OneWayInteraction() const { return MControlFlags.GetOneWayInteractionEnabled(); }
	void SetOneWayInteraction(bool bInOneWayInteraction) { MControlFlags.SetOneWayInteractionEnabled(bInOneWayInteraction); }

	bool InertiaConditioningEnabled() const { return MControlFlags.GetInertiaConditioningEnabled(); }
	void SetInertiaConditioningEnabled(bool bInEnabled) { MControlFlags.SetInertiaConditioningEnabled(bInEnabled); }

	FRigidParticleControlFlags ControlFlags() const { return MControlFlags; }
	void SetControlFlags(const FRigidParticleControlFlags& InFlags) { MControlFlags = InFlags; }

private:
	//NOTE: MObjectState is the only sim-writable data in this struct
	//If you add any more, make sure to update SyncSimWritablePropsFromSim
	//Or consider breaking it (and object state) out of this struct entirely
	FRealSingle MLinearEtherDrag;
	FRealSingle MAngularEtherDrag;
	FRealSingle MMaxLinearSpeedSq;
	FRealSingle MMaxAngularSpeedSq;
	FRealSingle MInitialOverlapDepenetrationVelocity = 0;
	FRealSingle MSleepThresholdMultiplier = 1;
	int32 MCollisionGroup;

	EObjectStateType MObjectState;
	EResimType MResimType;
	ESleepType MSleepType;

	uint32 MCollisionConstraintFlag = 0;
	FRigidParticleControlFlags MControlFlags;

	Chaos::Private::FIterationSettings MIterationSettings;

	bool bDisabled;
};

inline FChaosArchive& operator<<(FChaosArchive& Ar,FParticleDynamicMisc& Data)
{
	Data.Serialize(Ar);
	return Ar;
}

class FParticleNonFrequentData
{
public:
	FParticleNonFrequentData()
	{

	}

	void Serialize(FChaosArchive& Ar)
	{
		Ar.SerializePtr(MGeometry);
	}

	template <typename TOther>
	void CopyFrom(const TOther& Other)
	{
		SetGeometry(Other.GetGeometry());
		SetUniqueIdx(Other.UniqueIdx());
		SetSpatialIdx(Other.SpatialIdx());

		// Todo, can these be removed?
		SetResimType(Other.ResimType());
		SetEnabledDuringResim(Other.EnabledDuringResim());

#if CHAOS_DEBUG_NAME
		SetDebugName(Other.DebugName());
#endif
	}

	template <typename TOther>
	bool IsEqual(const TOther& Other) const
	{
		return GetGeometry() == Other.GetGeometry()
			&& UniqueIdx() == Other.UniqueIdx()
			&& SpatialIdx() == Other.SpatialIdx();
	}

	template <typename TOther>
	bool IsNearlyEqual(const TOther& Other, FReal Leniency = UE_KINDA_SMALL_NUMBER) const
	{
		// No properties that can be nearly compared
		return IsEqual(Other);
	}

	bool operator==(const FParticleNonFrequentData& Other) const
	{
		return IsEqual(Other);
	}

	//This function should only be used when geometry is not used by physics thread. The owning particle should not have a solver yet
	//Avoid using this function unless you know the threading model, see TGeometryParticle::ModifyGeometry
	FImplicitObject* AccessGeometryDangerous() { return const_cast<FImplicitObject*>(MGeometry.GetReference()); }

	const FImplicitObjectRef GetGeometry() const { return MGeometry.GetReference();}
	void SetGeometry(const FImplicitObjectPtr& InGeometry) { MGeometry = InGeometry;}

	UE_DEPRECATED(5.4, "Use GetGeometry instead")
	TSerializablePtr<FImplicitObject> Geometry() const { check(false); return TSerializablePtr<FImplicitObject>();}

	UE_DEPRECATED(5.4, "Use GetGeometry instead")
	const TSharedPtr<const FImplicitObject,ESPMode::ThreadSafe>& SharedGeometryLowLevel() const {  check(false); static TSharedPtr<const FImplicitObject, ESPMode::ThreadSafe> DummyPtr(nullptr); return DummyPtr;}

	UE_DEPRECATED(5.4, "Use SetGeometry with FImplicitObjectPtr instead")
	void SetGeometry(const TSharedPtr<const FImplicitObject,ESPMode::ThreadSafe>& InGeometry) { check(false); }
	
	const FUniqueIdx& UniqueIdx() const { return MUniqueIdx; }
	void SetUniqueIdx(FUniqueIdx InIdx){ MUniqueIdx = InIdx; }

	FSpatialAccelerationIdx SpatialIdx() const { return MSpatialIdx; }
	void SetSpatialIdx(FSpatialAccelerationIdx InIdx){ MSpatialIdx = InIdx; }

	EResimType ResimType() const { return MResimType; }

	void SetResimType(EResimType InType)
	{
		MResimType = InType;
	}

	void SetParticleID(const FParticleID& ParticleID)
	{
		MParticleID = ParticleID;
	}

	const FParticleID& ParticleID() const { return MParticleID; }

	bool EnabledDuringResim() const { return MEnabledDuringResim; }
	void SetEnabledDuringResim(bool bEnabledDuringResim) { MEnabledDuringResim = bEnabledDuringResim; }

#if CHAOS_DEBUG_NAME
	const TSharedPtr<FString, ESPMode::ThreadSafe>& DebugName() const { return MDebugName; }
	void SetDebugName(const TSharedPtr<FString, ESPMode::ThreadSafe>& InName) { MDebugName = InName; }
#endif
private:
	FImplicitObjectPtr MGeometry;
	FUniqueIdx MUniqueIdx;
	FSpatialAccelerationIdx MSpatialIdx;
	EResimType MResimType;
	bool MEnabledDuringResim;
	FParticleID MParticleID;
#if CHAOS_DEBUG_NAME
	TSharedPtr<FString, ESPMode::ThreadSafe> MDebugName;
#endif
};

inline FChaosArchive& operator<<(FChaosArchive& Ar,FParticleNonFrequentData& Data)
{
	Data.Serialize(Ar);
	return Ar;
}

struct FCollisionData
{
	UE_DEPRECATED(5.7, "Use Get/SetCombinedShapeFilterData instead")
	FCollisionFilterData QueryData;
	UE_DEPRECATED(5.7, "Use Get/SetCombinedShapeFilterData instead")
	FCollisionFilterData SimData;

	void* UserData;
	EChaosCollisionTraceFlag CollisionTraceType;
	uint8 bSimCollision : 1;
	uint8 bQueryCollision : 1;
	uint8 bIsProbe : 1;

	FCollisionData()
	: UserData(nullptr)
	, CollisionTraceType(EChaosCollisionTraceFlag::Chaos_CTF_UseDefault)
	, bSimCollision(true)
	, bQueryCollision(true)
	, bIsProbe(false)
	{
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FCollisionData(const FCollisionData& Rhs) = default;
	FCollisionData(FCollisionData& Rhs) = default;

	FCollisionData& operator=(const FCollisionData& Rhs) = default;
	FCollisionData& operator=(FCollisionData&& Rhs) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UE_INTERNAL CHAOS_API FCollisionFilterData GetQueryData() const;
	UE_INTERNAL CHAOS_API void SetQueryData(const FCollisionFilterData& InQueryData);
	UE_INTERNAL CHAOS_API FCollisionFilterData GetSimData() const;
	UE_INTERNAL CHAOS_API void SetSimData(const FCollisionFilterData& InSimData);

	CHAOS_API Chaos::Filter::FShapeFilterData GetShapeFilterData() const;
	CHAOS_API void SetShapeFilterData(const Chaos::Filter::FShapeFilterData& ShapeFilter);

	CHAOS_API Chaos::Filter::FInstanceData GetFilterInstanceData() const;
	CHAOS_API void SetFilterInstanceData(const Chaos::Filter::FInstanceData& InstanceData);

	CHAOS_API Chaos::Filter::FCombinedShapeFilterData GetCombinedShapeFilterData() const;
	CHAOS_API void SetCombinedShapeFilterData(const Chaos::Filter::FCombinedShapeFilterData& CombinedShapeFilter);

	bool HasCollisionData() const { return bSimCollision || bQueryCollision; }
	bool HasQueryOnlyData() const { return !bSimCollision && bQueryCollision; }

	void Serialize(FChaosArchive& Ar)
	{
		Ar.UsingCustomVersion(FExternalPhysicsCustomObjectVersion::GUID);
		Ar.UsingCustomVersion(FExternalPhysicsMaterialCustomObjectVersion::GUID);
		Ar.UsingCustomVersion(FFortniteSeasonBranchObjectVersion::GUID);
		Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::CollisionFilter64Bit)
		{
			FCollisionFilterData OldQueryData, OldSimData;
			if(Ar.IsLoading())
			{
				Ar << OldQueryData;
				Ar << OldSimData;
				const Chaos::Filter::FCombinedShapeFilterData CombinedFilter = Chaos::Filter::FShapeFilterBuilder::LoadFromLegacySerialization(OldQueryData, OldSimData);
				SetCombinedShapeFilterData(CombinedFilter);
			}
			else
			{
				Chaos::Filter::FCombinedShapeFilterData CombinedFilter = GetCombinedShapeFilterData();
				Chaos::Filter::FShapeFilterBuilder::GetLegacyShapeFilter(CombinedFilter, OldQueryData, OldSimData);
				Ar << OldQueryData;
				Ar << OldSimData;
			}
		}
		else
		{
			Chaos::Filter::FInstanceData InstanceData;
			Chaos::Filter::FShapeFilterData FilterData;
			if (Ar.IsLoading())
			{
				Ar << InstanceData;
				Ar << FilterData;
				SetFilterInstanceData(InstanceData);
				SetShapeFilterData(FilterData);
			}
			else
			{
				InstanceData = GetFilterInstanceData();
				FilterData = GetShapeFilterData();
				Ar << InstanceData;
				Ar << FilterData;
			}
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		if (Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) >= FExternalPhysicsCustomObjectVersion::AddShapeSimAndQueryCollisionEnabled)
		{
			int8 EnableSim = bSimCollision;
			int8 EnableQuery = bQueryCollision;
			Ar << EnableSim;
			Ar << EnableQuery;
			bSimCollision = EnableSim;
			bQueryCollision = EnableQuery;
		}
		else if (Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) >= FExternalPhysicsCustomObjectVersion::AddShapeCollisionDisable)
		{
			bool Disable = !bSimCollision;
			Ar << Disable;
			bSimCollision = !Disable;
		}

		if (Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) >= FExternalPhysicsCustomObjectVersion::SerializePerShapeDataSimulateFlag &&
			Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) < FExternalPhysicsCustomObjectVersion::RemoveShapeSimAndQueryDuplicateRepresentations)
		{
			bool Simulate = bSimCollision;
			Ar << Simulate;
			bSimCollision = Simulate;
		}

		if(Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) >= FExternalPhysicsCustomObjectVersion::SerializeCollisionTraceType)
		{
			int32 Data = (int32)CollisionTraceType;
			Ar << Data;
			CollisionTraceType = (EChaosCollisionTraceFlag)Data;
		}

		if (Ar.CustomVer(FFortniteSeasonBranchObjectVersion::GUID) >= FFortniteSeasonBranchObjectVersion::AddShapeIsProbe)
		{
			int8 IsProbe = bIsProbe;
			Ar << IsProbe;
			bIsProbe = IsProbe;
		}
	}
};

inline FChaosArchive& operator<<(FChaosArchive& Ar,FCollisionData& Data)
{
	//TODO: should this only work with dirty flag? Not sure if this path really matters at this point
	Data.Serialize(Ar);
	return Ar;
}

struct FMaterialData
{
	TArray<FMaterialHandle> Materials;
	TArray<FMaterialMaskHandle> MaterialMasks;
	TArray<uint32> MaterialMaskMaps;
	TArray<FMaterialHandle> MaterialMaskMapMaterials;

	void Serialize(FChaosArchive& Ar)
	{
		Ar.UsingCustomVersion(FExternalPhysicsCustomObjectVersion::GUID);
		Ar.UsingCustomVersion(FExternalPhysicsMaterialCustomObjectVersion::GUID);

		if(Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) >= FExternalPhysicsCustomObjectVersion::AddedMaterialManager)
		{
			Ar << Materials;
		}

		if(Ar.CustomVer(FExternalPhysicsMaterialCustomObjectVersion::GUID) >= FExternalPhysicsMaterialCustomObjectVersion::AddedMaterialMasks)
		{
			Ar << MaterialMasks << MaterialMaskMaps << MaterialMaskMapMaterials;
		}
	}
};

inline FChaosArchive& operator<<(FChaosArchive& Ar,FMaterialData& Data)
{
	//TODO: should this only work with dirty flag? Not sure if this path really matters at this point
	Data.Serialize(Ar);
	return Ar;
}

#define CHAOS_PROPERTY(PropName, Type, ProxyType) PropName,
	enum class EChaosProperty : uint32
	{
#include "ParticleProperties.inl"
		NumProperties
	};

#undef CHAOS_PROPERTY

#define CHAOS_PROPERTY(PropName, Type, ProxyType) PropName = (uint32)1 << (uint32)EChaosProperty::PropName,

	enum class EChaosPropertyFlags : uint32
	{
		#include "ParticleProperties.inl"
		DummyFlag
	};
#undef CHAOS_PROPERTY

	constexpr EChaosPropertyFlags ChaosPropertyToFlag(EChaosProperty Prop)
	{
		switch(Prop)
		{
#define CHAOS_PROPERTY(PropName, Type, ProxyType) case EChaosProperty::PropName: return EChaosPropertyFlags::PropName;
#include "ParticleProperties.inl"
#undef CHAOS_PROPERTY
		default: return (EChaosPropertyFlags)0;
		}
	}


#define SHAPE_PROPERTY(PropName, Type) PropName,
	enum class EShapeProperty: uint32
	{
#include "ShapeProperties.inl"
		NumShapeProperties
	};

#undef SHAPE_PROPERTY

#define SHAPE_PROPERTY(PropName, Type) PropName = (uint32)1 << (uint32)EShapeProperty::PropName,

	enum class EShapeFlags: uint32
	{
#include "ShapeProperties.inl"
		DummyFlag
	};
#undef SHAPE_PROPERTY

	constexpr EShapeFlags ShapePropToFlag(EShapeProperty Prop)
	{
		switch(Prop)
		{
#define SHAPE_PROPERTY(PropName, Type) case EShapeProperty::PropName: return EShapeFlags::PropName;
#include "ShapeProperties.inl"
#undef SHAPE_PROPERTY
		default: return (EShapeFlags)0;
		}
	}

	template <typename FlagsType>
	class TDirtyFlags
	{
	public:
		TDirtyFlags() : Bits(0) { }

		bool IsDirty() const
		{
			return Bits != 0;
		}

		bool IsDirty(const FlagsType CheckBits) const
		{
			return (Bits & (int32)CheckBits) != 0;
		}

		bool IsDirty(const int32 CheckBits) const
		{
			return (Bits & CheckBits) != 0;
		}

		void MarkDirty(const FlagsType DirtyBits)
		{
			Bits |= (int32)DirtyBits;
		}

		void MarkClean(const FlagsType CleanBits)
		{
			Bits &= ~(int32)CleanBits;
		}

		void Clear()
		{
			Bits = 0;
		}

		bool IsClean() const
		{
			return Bits == 0;
		}

		int32 GetBits() const
		{
			return Bits;
		}

		void SetBits(const int32 BitsIn)
		{
			Bits = BitsIn;
		}

	private:
		int32 Bits;
	};

	using FDirtyChaosPropertyFlags = TDirtyFlags<EChaosPropertyFlags>;
	using FShapeDirtyFlags = TDirtyFlags<EShapeFlags>;

	struct FDirtyIdx
	{
		uint32 bHasEntry : 1;
		uint32 Entry : 31;
	};

	template <typename T>
	class TDirtyElementPool
	{
	public:
		const T& GetElement(int32 Idx) const { return Elements[Idx]; }
		T& GetElement(int32 Idx){ return Elements[Idx]; }
		
		void Reset(int32 Idx)
		{
			Elements[Idx] = T();
		}

		void SetNum(int32 Num)
		{
			Elements.SetNum(Num);
		}

		int32 Num() const
		{
			return Elements.Num();
		}

	private:

		TArray<T> Elements;
	};


/** Helper struct to let us know how many proxies are dirty per type,
  * as well as how to go from a contiguous index into a per bucket index */
struct FDirtyProxiesBucketInfo
{
	int32 Num[(uint32)(EPhysicsProxyType::Count)] = {};
	int32 TotalNum = 0;

	void Reset()
	{
		for (int32 Idx = 0; Idx < (uint32)EPhysicsProxyType::Count; ++Idx) { Num[Idx] = 0; }
		TotalNum = 0;
	}

	void GetBucketIdx(int32 Idx, int32& OutBucketIdx, int32& InnerIdx) const
	{
		int32 Remaining = Idx;
		for (int32 BucketIdx = 0; BucketIdx < (uint32)EPhysicsProxyType::Count; ++BucketIdx)
		{
			if (Remaining < Num[BucketIdx])
			{
				InnerIdx = Remaining;
				OutBucketIdx = BucketIdx;
				return;
			}
			else
			{
				Remaining -= Num[BucketIdx];
			}
		}

		check(false);	//couldn't find bucket for the given index
	}
};


class FDirtyPropertiesManager
{
public:

	void PrepareBuckets(const FDirtyProxiesBucketInfo& DirtyProxiesBucketInfo)
	{
#define CHAOS_PROPERTY(PropName, Type, ProxyType) PropName##Pool.SetNum(DirtyProxiesBucketInfo.Num[(uint32)ProxyType]);
#include "ParticleProperties.inl"
#undef CHAOS_PROPERTY
	}

	void SetNumShapes(int32 NumShapes)
	{
#define SHAPE_PROPERTY(PropName, Type) PropName##ShapePool.SetNum(NumShapes);
#include "ShapeProperties.inl"
#undef SHAPE_PROPERTY
	}

	template <typename T, EChaosProperty PropName>
	TDirtyElementPool<T>& GetChaosPropertyPool()
	{
		switch(PropName)
		{
#define CHAOS_PROPERTY(PropName, Type, ProxyType) case EChaosProperty::PropName: return (TDirtyElementPool<T>&)PropName##Pool;
#include "ParticleProperties.inl"
#undef CHAOS_PROPERTY
		default: check(false);
		}

		static TDirtyElementPool<T> ErrorPool;
		return ErrorPool;
	}

	template <typename T,EChaosProperty PropName>
	const TDirtyElementPool<T>& GetChaosPropertyPool() const
	{
		switch(PropName)
		{
#define CHAOS_PROPERTY(PropName, Type, ProxyType) case EChaosProperty::PropName: return (TDirtyElementPool<T>&)PropName##Pool;
#include "ParticleProperties.inl"
#undef CHAOS_PROPERTY
		default: check(false);
		}

		static TDirtyElementPool<T> ErrorPool;
		return ErrorPool;
	}

	template <typename T,EShapeProperty PropName>
	TDirtyElementPool<T>& GetShapePool()
	{
		switch(PropName)
		{
#define SHAPE_PROPERTY(PropName, Type) case EShapeProperty::PropName: return (TDirtyElementPool<T>&)PropName##ShapePool;
#include "ShapeProperties.inl"
#undef SHAPE_PROPERTY
		default: check(false);
		}

		static TDirtyElementPool<T> ErrorPool;
		return ErrorPool;
	}

	template <typename T,EShapeProperty PropName>
	const TDirtyElementPool<T>& GetShapePool() const
	{
		switch(PropName)
		{
#define SHAPE_PROPERTY(PropName, Type) case EShapeProperty::PropName: return (TDirtyElementPool<T>&)PropName##ShapePool;
#include "ShapeProperties.inl"
#undef SHAPE_PROPERTY
		default: check(false);
		}

		static TDirtyElementPool<T> ErrorPool;
		return ErrorPool;
	}

private:

#define CHAOS_PROPERTY(PropName, Type, ProxyType) TDirtyElementPool<Type> PropName##Pool;
#include "ParticleProperties.inl"
#undef CHAOS_PROPERTY

#define SHAPE_PROPERTY(PropName, Type) TDirtyElementPool<Type> PropName##ShapePool;
#include "ShapeProperties.inl"
#undef SHAPE_PROPERTY

};

class FDirtyChaosProperties
{
public:
	
	void SetParticleBufferType(EParticleType Type)
	{
		ParticleBufferType = Type;
	}

	//NOTE: this is only valid if the proxy is a particle type and SetParticleBufferType was used
	//TODO: remove this from API
	EParticleType GetParticleBufferType() const
	{
		return ParticleBufferType;
	}

	void SetFlags(FDirtyChaosPropertyFlags InFlags)
	{
		Flags = InFlags;
	}

	FDirtyChaosPropertyFlags GetFlags() const
	{
		return Flags;
	}

	void DirtyFlag(EChaosPropertyFlags Flag)
	{
		Flags.MarkDirty(Flag);
	}

	template <typename T, EChaosProperty PropName>
	void SyncRemote(FDirtyPropertiesManager& Manager, int32 Idx, const T& Val) const
	{
		if(Flags.IsDirty(ChaosPropertyToFlag(PropName)))
		{
			Manager.GetChaosPropertyPool<T,PropName>().GetElement(Idx) = Val;
		}
	}

	void Clear(FDirtyPropertiesManager& Manager, int32 Idx)
	{
#define CHAOS_PROPERTY(PropName, Type, ProxyType) ClearHelper<Type, EChaosProperty::PropName>(Manager, Idx);
#include "ParticleProperties.inl"
#undef CHAOS_PROPERTY
		Flags.Clear();
	}

	bool IsDirty(EChaosPropertyFlags InBits) const
	{
		return Flags.IsDirty(InBits);
	}

	void MoveData(FDirtyPropertiesManager& Manager, int32 FromIdx, int32 ToIdx)
	{
#define CHAOS_PROPERTY(PropName, Type, ProxyType) MoveDataHelper<Type, EChaosProperty::PropName>(Manager, FromIdx, ToIdx);
#include "ParticleProperties.inl"
#undef CHAOS_PROPERTY
	}

#define CHAOS_PROPERTY(PropName, Type, ProxyType)\
Type const & Get##PropName(const FDirtyPropertiesManager& Manager, int32 Idx) const { return ReadImp<Type, EChaosProperty::PropName>(Manager, Idx); }\
bool Has##PropName() const { return Flags.IsDirty(ChaosPropertyToFlag(EChaosProperty::PropName)); }\
Type const * Find##PropName(const FDirtyPropertiesManager& Manager, int32 Idx) const { return Has##PropName() ? &Get##PropName(Manager, Idx) : nullptr; }

#include "ParticleProperties.inl"
#undef CHAOS_PROPERTY

private:
	FDirtyChaosPropertyFlags Flags;
	EParticleType ParticleBufferType;

	template <typename T,EChaosProperty PropName>
	const T& ReadImp(const FDirtyPropertiesManager& Manager, int32 Idx) const
	{
		ensure(Flags.IsDirty(ChaosPropertyToFlag(PropName)));
		return Manager.GetChaosPropertyPool<T,PropName>().GetElement(Idx);
	}

	template <typename T, EChaosProperty PropName>
	void ClearHelper(FDirtyPropertiesManager& Manager, int32 Idx)
	{
		if(Flags.IsDirty(ChaosPropertyToFlag(PropName)))
		{
			Manager.GetChaosPropertyPool<T, PropName>().Reset(Idx);
		}
	}

	template <typename T, EChaosProperty PropName>
	void MoveDataHelper(FDirtyPropertiesManager& Manager, int32 FromIdx, int32 ToIdx)
	{
		if (Flags.IsDirty(ChaosPropertyToFlag(PropName)))
		{
			Manager.GetChaosPropertyPool<T, PropName>().GetElement(ToIdx) = Manager.GetChaosPropertyPool<T, PropName>().GetElement(FromIdx);
			Manager.GetChaosPropertyPool<T, PropName>().Reset(FromIdx);
		}
	}
};

class FShapeDirtyData
{
public:

	FShapeDirtyData(int32 InShapeIdx)
	: ShapeIdx(InShapeIdx)
	{

	}

	int32 GetShapeIdx() const { return ShapeIdx; }

	void SetFlags(FShapeDirtyFlags InFlags)
	{
		Flags = InFlags;
	}

	template <typename T,EShapeProperty PropName>
	void SyncRemote(FDirtyPropertiesManager& Manager,int32 Idx, const T& Val) const
	{
		if(Flags.IsDirty(ShapePropToFlag(PropName)))
		{
			Manager.GetShapePool<T,PropName>().GetElement(Idx) = Val;
		}
	}

	template <EShapeProperty PropName>
	bool IsDirty() const
	{
		return Flags.IsDirty(ShapePropToFlag(PropName));
	}

	void Clear(FDirtyPropertiesManager& Manager, int32 Idx)
	{
#define SHAPE_PROPERTY(PropName, Type) ClearHelper<Type, EShapeProperty::PropName>(Manager, Idx);
#include "ShapeProperties.inl"
#undef SHAPE_PROPERTY
		Flags.Clear();
	}

#define SHAPE_PROPERTY(PropName, Type)\
Type const & Get##PropName(const FDirtyPropertiesManager& Manager, int32 Idx) const { return ReadImp<Type, EShapeProperty::PropName>(Manager, Idx); }\
bool Has##PropName() const { return Flags.IsDirty(ShapePropToFlag(EShapeProperty::PropName)); }\
Type const * Find##PropName(const FDirtyPropertiesManager& Manager, int32 Idx) const { return Has##PropName() ? &Get##PropName(Manager, Idx) : nullptr; }

#include "ShapeProperties.inl"
#undef CHAOS_PROPERTY

private:
	int32 ShapeIdx;
	FShapeDirtyFlags Flags;

	template <typename T,EShapeProperty PropName>
	const T& ReadImp(const FDirtyPropertiesManager& Manager, int32 Idx) const
	{
		ensure(Flags.IsDirty(ShapePropToFlag(PropName)));
		return Manager.GetShapePool<T,PropName>().GetElement(Idx);
	}

	template <typename T,EShapeProperty PropName>
	void ClearHelper(FDirtyPropertiesManager& Manager, int32 Idx)
	{
		if(Flags.IsDirty(ShapePropToFlag(PropName)))
		{
			Manager.GetShapePool<T,PropName>().Reset(Idx);
		}
	}
};

template <typename T>
class TPropertyPool;

using FPropertyIdx = int32;

template <typename T>
class TPropertyPool
{
public:

	T& AddElement(FPropertyIdx& OutIdx)
	{
		if(FreeList.Num())
		{
			OutIdx = FreeList.Pop();
			return Elements[OutIdx];
		}
		else
		{
			OutIdx = Elements.AddDefaulted(1);
			return Elements[OutIdx];
		}
	}

	void RemoveElement(const FPropertyIdx Idx)
	{
		Elements[Idx] = T();
		FreeList.Push(Idx);
	}

	T& GetElement(const FPropertyIdx Idx)
	{
		return Elements[Idx];
	}

	const T& GetElement(const FPropertyIdx Idx) const
	{
		return Elements[Idx];
	}

	~TPropertyPool()
	{
		ensure(Elements.Num() == FreeList.Num());	//All elements have been freed
	}

private:

	TArray<T> Elements;
	TArray<FPropertyIdx> FreeList;
};

//Similar to FDirtyPropertiesManager but is not needed to be used across threads
//This means we just have one big pool per property that you can new/free into
class FDirtyPropertiesPool
{
public:
	template <typename T, EChaosProperty PropName>
	TPropertyPool<T>& GetPool()
	{
		switch (PropName)
		{
#define CHAOS_PROPERTY(PropName, Type, ProxyType) case EChaosProperty::PropName: return (TPropertyPool<T>&)PropName##Pool;
#include "ParticleProperties.inl"
#undef CHAOS_PROPERTY
		default: check(false);
		}

		static TPropertyPool<T> ErrorPool;
		return ErrorPool;
	}

	template <typename T, EChaosProperty PropName>
	const TPropertyPool<T>& GetPool() const
	{
		switch (PropName)
		{
#define CHAOS_PROPERTY(PropName, Type, ProxyType) case EChaosProperty::PropName: return (TPropertyPool<T>&)PropName##Pool;
#include "ParticleProperties.inl"
#undef CHAOS_PROPERTY
		default: check(false);
		}

		static TPropertyPool<T> ErrorPool;
		return ErrorPool;
	}

private:

#define CHAOS_PROPERTY(PropName, Type, ProxyType) TPropertyPool<Type> PropName##Pool;
#include "ParticleProperties.inl"
#undef CHAOS_PROPERTY
};
}
